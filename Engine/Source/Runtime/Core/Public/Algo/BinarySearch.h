// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/Less.h"

namespace AlgoImpl
{
	/**
	 * Performs binary search, resulting in position of the first element >= Value
	 *
	 * @param First Pointer to array
	 * @param Num Number of elements in array
	 * @param Value Value to look for
	 * @param Projection Called on values in array to get type that can be compared to Value
	 * @param SortPredicate Predicate for sort comparison
	 *
	 * @returns Position of the first element >= Value, may be == Num
	 */
	template <typename RangeValueType, typename SizeType, typename PredicateValueType, typename ProjectionType, typename SortPredicateType>
	SizeType LowerBoundInternal(RangeValueType* First, const SizeType Num, const PredicateValueType& Value, ProjectionType Projection, SortPredicateType SortPredicate)
	{
		// Current start of sequence to check
		SizeType Start = 0;
		// Size of sequence to check
		SizeType Size = Num;

		// With this method, if Size is even it will do one more comparison than necessary, but because Size can be predicted by the CPU it is faster in practice
		while (Size > 0)
		{
			const SizeType LeftoverSize = Size % 2;
			Size = Size / 2;

			const SizeType CheckIndex = Start + Size;
			const SizeType StartIfLess = CheckIndex + LeftoverSize;

			auto&& CheckValue = Invoke(Projection, First[CheckIndex]);
			Start = SortPredicate(CheckValue, Value) ? StartIfLess : Start;
		}
		return Start;
	}

	/**
	 * Performs binary search, resulting in position of the first element that is larger than the given value
	 *
	 * @param First Pointer to array
	 * @param Num Number of elements in array
	 * @param Value Value to look for
	 * @param SortPredicate Predicate for sort comparison
	 *
	 * @returns Position of the first element > Value, may be == Num
	 */
	template <typename RangeValueType, typename SizeType, typename PredicateValueType, typename ProjectionType, typename SortPredicateType>
	SizeType UpperBoundInternal(RangeValueType* First, const SizeType Num, const PredicateValueType& Value, ProjectionType Projection, SortPredicateType SortPredicate)
	{
		// Current start of sequence to check
		SizeType Start = 0;
		// Size of sequence to check
		SizeType Size = Num;

		// With this method, if Size is even it will do one more comparison than necessary, but because Size can be predicted by the CPU it is faster in practice
		while (Size > 0)
		{
			const SizeType LeftoverSize = Size % 2;
			Size = Size / 2;

			const SizeType CheckIndex = Start + Size;
			const SizeType StartIfLess = CheckIndex + LeftoverSize;

			auto&& CheckValue = Invoke(Projection, First[CheckIndex]);
			Start = !SortPredicate(Value, CheckValue) ? StartIfLess : Start;
		}

		return Start;
	}
}

namespace Algo
{
	/**
	 * Performs binary search, resulting in position of the first element >= Value using predicate
	 *
	 * @param Range Range to search through, must be already sorted by SortPredicate
	 * @param Value Value to look for
	 * @param SortPredicate Predicate for sort comparison, defaults to <
	 *
	 * @returns Position of the first element >= Value, may be position after last element in range
	 */
	template <typename RangeType, typename ValueType, typename SortPredicateType>
	[[nodiscard]] UE_REWRITE auto LowerBound(const RangeType& Range, const ValueType& Value, SortPredicateType SortPredicate) -> decltype(GetNum(Range))
	{
		return AlgoImpl::LowerBoundInternal(GetData(Range), GetNum(Range), Value, FIdentityFunctor(), SortPredicate);
	}
	template <typename RangeType, typename ValueType>
	[[nodiscard]] UE_REWRITE auto LowerBound(const RangeType& Range, const ValueType& Value) -> decltype(GetNum(Range))
	{
		return AlgoImpl::LowerBoundInternal(GetData(Range), GetNum(Range), Value, FIdentityFunctor(), TLess<>());
	}

	/**
	 * Performs binary search, resulting in position of the first element with projected value >= Value using predicate
	 *
	 * @param Range Range to search through, must be already sorted by SortPredicate
	 * @param Value Value to look for
	 * @param Projection Functor or data member pointer, called via Invoke to compare to Value
	 * @param SortPredicate Predicate for sort comparison, defaults to <
	 *
	 * @returns Position of the first element >= Value, may be position after last element in range
	 */
	template <typename RangeType, typename ValueType, typename ProjectionType, typename SortPredicateType>
	[[nodiscard]] UE_REWRITE auto LowerBoundBy(const RangeType& Range, const ValueType& Value, ProjectionType Projection, SortPredicateType SortPredicate) -> decltype(GetNum(Range))
	{
		return AlgoImpl::LowerBoundInternal(GetData(Range), GetNum(Range), Value, Projection, SortPredicate);
	}
	template <typename RangeType, typename ValueType, typename ProjectionType>
	[[nodiscard]] UE_REWRITE auto LowerBoundBy(const RangeType& Range, const ValueType& Value, ProjectionType Projection) -> decltype(GetNum(Range))
	{
		return AlgoImpl::LowerBoundInternal(GetData(Range), GetNum(Range), Value, Projection, TLess<>());
	}

	/**
	 * Performs binary search, resulting in position of the first element > Value using predicate
	 *
	 * @param Range Range to search through, must be already sorted by SortPredicate
	 * @param Value Value to look for
	 * @param SortPredicate Predicate for sort comparison, defaults to <
	 *
	 * @returns Position of the first element > Value, may be past end of range
	 */
	template <typename RangeType, typename ValueType, typename SortPredicateType>
	[[nodiscard]] UE_REWRITE auto UpperBound(const RangeType& Range, const ValueType& Value, SortPredicateType SortPredicate) -> decltype(GetNum(Range))
	{
		return AlgoImpl::UpperBoundInternal(GetData(Range), GetNum(Range), Value, FIdentityFunctor(), SortPredicate);
	}
	template <typename RangeType, typename ValueType>
	[[nodiscard]] UE_REWRITE auto UpperBound(const RangeType& Range, const ValueType& Value) -> decltype(GetNum(Range))
	{
		return AlgoImpl::UpperBoundInternal(GetData(Range), GetNum(Range), Value, FIdentityFunctor(), TLess<>());
	}

	/**
	 * Performs binary search, resulting in position of the first element with projected value > Value using predicate
	 *
	 * @param Range Range to search through, must be already sorted by SortPredicate
	 * @param Value Value to look for
	 * @param Projection Functor or data member pointer, called via Invoke to compare to Value
	 * @param SortPredicate Predicate for sort comparison, defaults to <
	 *
	 * @returns Position of the first element > Value, may be past end of range
	 */
	template <typename RangeType, typename ValueType, typename ProjectionType, typename SortPredicateType>
	[[nodiscard]] UE_REWRITE auto UpperBoundBy(const RangeType& Range, const ValueType& Value, ProjectionType Projection, SortPredicateType SortPredicate) -> decltype(GetNum(Range))
	{
		return AlgoImpl::UpperBoundInternal(GetData(Range), GetNum(Range), Value, Projection, SortPredicate);
	}
	template <typename RangeType, typename ValueType, typename ProjectionType>
	[[nodiscard]] UE_REWRITE auto UpperBoundBy(const RangeType& Range, const ValueType& Value, ProjectionType Projection) -> decltype(GetNum(Range))
	{
		return AlgoImpl::UpperBoundInternal(GetData(Range), GetNum(Range), Value, Projection, TLess<>());
	}

	/**
	 * Returns index to the first found element matching a value in a range, the range must be sorted by <
	 *
	 * @param Range The range to search, must be already sorted by SortPredicate
	 * @param Value The value to search for
	 * @param SortPredicate Predicate for sort comparison, defaults to <
	 * @return Index of found element, or INDEX_NONE
	 */
	template <typename RangeType, typename ValueType, typename SortPredicateType>
	[[nodiscard]] auto BinarySearch(const RangeType& Range, const ValueType& Value, SortPredicateType SortPredicate) -> decltype(GetNum(Range))
	{
		auto CheckIndex = LowerBound(Range, Value, SortPredicate);
		if (CheckIndex < GetNum(Range))
		{
			auto&& CheckValue = GetData(Range)[CheckIndex];
			// Since we returned lower bound we already know Value <= CheckValue. So if Value is not < CheckValue, they must be equal
			if (!SortPredicate(Value, CheckValue))
			{
				return CheckIndex;
			}
		}
		return INDEX_NONE;
	}
	template <typename RangeType, typename ValueType>
	[[nodiscard]] UE_REWRITE auto BinarySearch(const RangeType& Range, const ValueType& Value)
	{
		return BinarySearch(Range, Value, TLess<>());
	}

	/**
	 * Returns index to the first found element with projected value matching Value in a range, the range must be sorted by predicate
	 *
	 * @param Range The range to search, must be already sorted by SortPredicate
	 * @param Value The value to search for
	 * @param Projection Functor or data member pointer, called via Invoke to compare to Value
	 * @param SortPredicate Predicate for sort comparison, defaults to <
	 * @return Index of found element, or INDEX_NONE
	 */
	template <typename RangeType, typename ValueType, typename ProjectionType, typename SortPredicateType>
	[[nodiscard]] auto BinarySearchBy(const RangeType& Range, const ValueType& Value, ProjectionType Projection, SortPredicateType SortPredicate) -> decltype(GetNum(Range))
	{
		auto CheckIndex = LowerBoundBy(Range, Value, Projection, SortPredicate);
		if (CheckIndex < GetNum(Range))
		{
			auto&& CheckValue = Invoke(Projection, GetData(Range)[CheckIndex]);
			// Since we returned lower bound we already know Value <= CheckValue. So if Value is not < CheckValue, they must be equal
			if (!SortPredicate(Value, CheckValue))
			{
				return CheckIndex;
			}
		}
		return INDEX_NONE;
	}
	template <typename RangeType, typename ValueType, typename ProjectionType>
	[[nodiscard]] UE_REWRITE auto BinarySearchBy(const RangeType& Range, const ValueType& Value, ProjectionType Projection)
	{
		return BinarySearchBy(Range, Value, Projection, TLess<>());
	}
}
