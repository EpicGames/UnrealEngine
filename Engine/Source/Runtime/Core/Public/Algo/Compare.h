// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/EqualTo.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/Less.h"
#include "Templates/UnrealTemplate.h"

namespace Algo::Private
{
	template <typename InAT, typename InBT, typename ProjectionT, typename PredicateT>
	constexpr bool Compare(const InAT& InputA, const InBT& InputB, ProjectionT Projection, PredicateT Predicate)
	{
		const SIZE_T SizeA = GetNum(InputA);
		const SIZE_T SizeB = GetNum(InputB);

		if (SizeA != SizeB)
		{
			return false;
		}

		auto* A = GetData(InputA);
		auto* B = GetData(InputB);

		for (SIZE_T Count = SizeA; Count; --Count)
		{
			if (!Invoke(Predicate, Invoke(Projection, *A++), Invoke(Projection, *B++)))
			{
				return false;
			}
		}

		return true;
	}
}

namespace Algo
{
	/**
	 * Compares two contiguous containers using operator== to compare pairs of elements.
	 *
	 * @param  InputA     Container of elements that are used as the first argument to operator==.
	 * @param  InputB     Container of elements that are used as the second argument to operator==.
	 *
	 * @return Whether the containers are the same size and operator== returned true for every pair of elements.
	 */
	template <typename InAT, typename InBT>
	[[nodiscard]] constexpr bool Compare(const InAT& InputA, const InBT& InputB)
	{
		return Private::Compare(InputA, InputB, FIdentityFunctor(), TEqualTo<>());
	}

	/**
	 * Compares two contiguous containers using a predicate to compare pairs of elements.
	 *
	 * @param  InputA     Container of elements that are used as the first argument to the predicate.
	 * @param  InputB     Container of elements that are used as the second argument to the predicate.
	 * @param  Predicate  Condition which returns true for elements which are deemed equal.
	 *
	 * @return Whether the containers are the same size and the predicate returned true for every pair of elements.
	 */
	template <typename InAT, typename InBT, typename PredicateT>
	[[nodiscard]] constexpr bool Compare(const InAT& InputA, const InBT& InputB, PredicateT Predicate)
	{
		return Private::Compare(InputA, InputB, FIdentityFunctor(), MoveTemp(Predicate));
	}

	/**
	 * Compares two contiguous containers using operator== to compare pairs of projected elements.
	 *
	 * @param  InputA     Container of elements that are used as the first argument to operator==.
	 * @param  InputB     Container of elements that are used as the second argument to operator==.
	 * @param  Projection Projection to apply to the elements before comparing them.
	 *
	 * @return Whether the containers are the same size and operator== returned true for every pair of elements.
	 */
	template <typename InAT, typename InBT, typename ProjectionT>
	[[nodiscard]] constexpr bool CompareBy(const InAT& InputA, const InBT& InputB, ProjectionT Projection)
	{
		return Private::Compare(InputA, InputB, MoveTemp(Projection), TEqualTo<>());
	}

	/**
	 * Compares two contiguous containers using a predicate to compare pairs of projected elements.
	 *
	 * @param  InputA     Container of elements that are used as the first argument to the predicate.
	 * @param  InputB     Container of elements that are used as the second argument to the predicate.
	 * @param  Projection Projection to apply to the elements before comparing them.
	 * @param  Predicate  Condition which returns true for elements which are deemed equal.
	 *
	 * @return Whether the containers are the same size and the predicate returned true for every pair of elements.
	 */
	template <typename InAT, typename InBT, typename ProjectionT, typename PredicateT>
	[[nodiscard]] constexpr bool CompareBy(const InAT& InputA, const InBT& InputB, ProjectionT Projection, PredicateT Predicate)
	{
		return Private::Compare(InputA, InputB, MoveTemp(Projection), MoveTemp(Predicate));
	}

	/**
	 * Compares two unique-key maps (e.g. TMap) as if they were sorted arrays of key,value pairs sorted by Key and then
	 * by Value (almost, see note on sort order). Does not support multiple values per key (e.g. TMultiMap).
	 * 
	 * Note on sort order: Maps with a smaller number of elements are considered less than maps with a larger number
	 * of elements, no matter what keys are present in each map. This drastically improves performance when comparing
	 * maps of different sizes. This is different than would be exepcted from a lexical compare of strings, but it does
	 * match the comparison of two numbers represented as a string of digits.
	 * 
	 * MapType interface:
	 *  typename KeyType, typename ValueType,
	 *  IntType Num(), ValueType* Find(const KeyType&), IteratorType<PairType> begin(), end()
	 * PairType interface:
	 *  const KeyType& Key, const ValueType& Value
	 * 
	 * @param A Left-Hand-Side map container.
	 * @param B Right-Hand-Side map container.
	 * @param KeyLessThan bool(const KeyType& A, const KeyType& B) that returns A < B.
	 * @param ValueLessThan bool(const ValueType& A, const ValueType& B) that returns A < B.
	 *
	 * @return -1 if A < B, 0 if A == B, 1 if A > B
	 */
	template <typename MapType, typename KeyLessThanType, typename ValueLessThanType>
	[[nodiscard]] int CompareMap(const MapType& A, const MapType& B, KeyLessThanType KeyLessThan, ValueLessThanType ValueLessThan)
	{
		using KeyType = typename MapType::KeyType;
		using ValueType = typename MapType::ValueType;
		if (A.Num() != B.Num())
		{
			return A.Num() < B.Num() ? -1 : 1;
		}
		if (A.Num() == 0)
		{
			return 0;
		}

		bool bAllKeysOfAAreInB = true;
		const KeyType* MinKeyWithDifference = nullptr;
		bool bMinKeyIsLessInA = false;
		for (const auto& Pair : A) // TPair<KeyType, ValueType>
		{
			const ValueType* BValue = B.Find(Pair.Key);
			int Compare = 0;
			if (!BValue)
			{
				bAllKeysOfAAreInB = false;
				Compare = -1;
			}
			else if (ValueLessThan(Pair.Value, *BValue))
			{
				Compare = -1;
			}
			else if (ValueLessThan(*BValue, Pair.Value))
			{
				Compare = 1;
			}
			if (Compare != 0)
			{
				if (!MinKeyWithDifference || KeyLessThan(Pair.Key, *MinKeyWithDifference))
				{
					MinKeyWithDifference = &Pair.Key;
					bMinKeyIsLessInA = Compare < 0;
				}
			}
		}

		// The number of keys in A and B is the same (checked above), so if all keys of A are in B, then there are
		// no additional keys in B that are not in A and we don't need to iterate over B.
		if (!bAllKeysOfAAreInB)
		{
			// If B has additional keys not in A then we need to check each of those not-in-A keys to see if they
			// are smaller than the MinKeyWithDifference, and if so they become the MinKeyWithDifference.
			for (const auto& Pair : B) // TPair<KeyType, ValueType>
			{
				if (!A.Contains(Pair.Key))
				{
					if (!MinKeyWithDifference || KeyLessThan(Pair.Key, *MinKeyWithDifference))
					{
						MinKeyWithDifference = &Pair.Key;
						bMinKeyIsLessInA = false;
					}
				}
			}
		}
		if (MinKeyWithDifference)
		{
			return bMinKeyIsLessInA ? -1 : 1;
		}
		return 0;
	}

	template <typename MapType>
	[[nodiscard]] int CompareMap(const MapType& A, const MapType& B)
	{
		return CompareMap(A, B, TLess<>(), TLess<>());
	}

	template <typename MapType, typename KeyLessThanType>
	[[nodiscard]] int CompareMap(const MapType& A, const MapType& B, KeyLessThanType KeyLessThan)
	{
		return CompareMap(A, B, MoveTempIfPossible(KeyLessThan), TLess<>());
	}

	/**
	 * Compares two sets (e.g. TSet) as if they were sorted arrays of keys (almost, see note on sort order).
	 *
	 * Note on sort order: Maps with a smaller number of elements are considered less than maps with a larger number
	 * of elements, no matter what keys are present in each map. This drastically improves performance when comparing
	 * maps of different sizes. This is different than would be exepcted from a lexical compare of strings, but it does
	 * match the comparison of two numbers represented as a string of digits.
	 *
	 * SetType interface:
	 *  typename ElementType
	 *  IntType Num(), bool Contains(const KeyType&), IteratorType<KeyType> begin(), end()
	 *
	 * @param A Left-Hand-Side set container.
	 * @param B Right-Hand-Side set container.
	 * @param KeyLessThan bool(const KeyType& A, const KeyType& B) that returns A < B.
	 *
	 * @return -1 if A < B, 0 if A == B, 1 if A > B
	 */
	template <typename SetType, typename KeyLessThanType>
	[[nodiscard]] int CompareSet(const SetType& A, const SetType& B, KeyLessThanType KeyLessThan)
	{
		using KeyType = typename SetType::ElementType;
		if (A.Num() != B.Num())
		{
			return A.Num() < B.Num() ? -1 : 1;
		}
		if (A.Num() == 0)
		{
			return 0;
		}

		const KeyType* MinKeyWithDifference = nullptr;
		bool bMinKeyIsLessInA = false;
		for (const KeyType& AKey : A)
		{
			if (!B.Contains(AKey))
			{
				if (!MinKeyWithDifference || KeyLessThan(AKey, *MinKeyWithDifference))
				{
					MinKeyWithDifference = &AKey;
					bMinKeyIsLessInA = true;
				}
			}
		}

		// The number of keys in A and B is the same (checked above), so if all keys of A are in B, then there are
		// no additional keys in B that are not in A and we don't need to iterate over B.
		if (MinKeyWithDifference)
		{
			// If B has additional keys not in A then we need to check each of those not-in-A keys to see if they
			// are smaller than the MinKeyWithDifference, and if so they become the MinKeyWithDifference.
			for (const KeyType& BKey : B)
			{
				if (!A.Contains(BKey))
				{
					if (!MinKeyWithDifference || KeyLessThan(BKey, *MinKeyWithDifference))
					{
						MinKeyWithDifference = &BKey;
						bMinKeyIsLessInA = false;
					}
				}
			}
		}
		if (MinKeyWithDifference)
		{
			return bMinKeyIsLessInA ? -1 : 1;
		}
		return 0;
	}

	template <typename SetType>
	[[nodiscard]] int CompareSet(const SetType& A, const SetType& B)
	{
		return CompareSet(A, B, TLess<>());
	}
}
