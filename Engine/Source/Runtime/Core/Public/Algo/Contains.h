// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Find.h"
#include "Templates/UnrealTemplate.h" // For MoveTemp and Forward

namespace Algo
{
	/**
	 * Checks if the value exists in the range.
	 *
	 * @param  Range  The range to search.
	 * @param  Value  The value to search for.
	 *
	 * @return true if an element was found, false otherwise.
	 */
	template <typename RangeType, typename ValueType>
	[[nodiscard]] UE_REWRITE constexpr bool Contains(RangeType&& Range, const ValueType& Value)
	{
		return !!Algo::Find(Forward<RangeType>(Range), Value);
	}

	/**
	 * Checks if the value exists in the range given by the projection.
	 *
	 * @param  Range  The range to search.
	 * @param  Value  The value to search for.
	 * @param  Proj   The projection to apply to the element.
	 *
	 * @return true if an element was found, false otherwise.
	 */
	template <typename RangeType, typename ValueType, typename ProjectionType>
	[[nodiscard]] UE_REWRITE constexpr bool ContainsBy(RangeType&& Range, const ValueType& Value, ProjectionType Proj)
	{
		return !!Algo::FindBy(Forward<RangeType>(Range), Value, MoveTemp(Proj));
	}

	// There is no Algo::ContainsByPredicate as it would be identical to Algo::AnyOf.
	// This comment exists to guide users to Algo::AnyOf.
}
