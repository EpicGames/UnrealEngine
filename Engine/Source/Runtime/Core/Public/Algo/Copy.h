// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Algo/Common.h"

namespace Algo
{
	/**
	 * Conditionally copies a range into a container
	 *
	 * @param  Input  Any iterable type
	 * @param  Output  Container to hold the output
	 * @param  Predicate  Condition which returns true for elements that should be copied and false for elements that should be skipped
	 */
	template <typename InT, typename OutT, typename PredicateT>
	void CopyIf(const InT& Input, OutT& Output, PredicateT Predicate)
	{
		for (const auto& Value : Input)
		{
			if (Invoke(Predicate, Value))
			{
				Output.Add(Value);
			}
		}
	}

	/**
	 * Copies a range into a container
	 *
	 * @param  Input  Any iterable type
	 * @param  Output  Container to hold the output
	 */
	template <typename InT, typename OutT>
	void Copy(const InT& Input, OutT& Output)
	{
		for (const auto& Value : Input)
		{
			Output.Add(Value);
		}
	}

	/**
	 * Copies a range into a container.
	 * Should be used if when the input iterator doesn't return a reference.
	 *
	 * @param  Input  Any iterable type
	 * @param  Output  Container to hold the output
	 */
	template <typename InT, typename OutT>
	void Copy(const InT& Input, OutT& Output, ENoRef NoRef)
	{
		for (const auto Value : Input)
		{
			Output.Add(Value);
		}
	}
}
