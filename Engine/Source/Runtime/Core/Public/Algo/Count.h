// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace Algo
{
	/**
	 * Counts elements of a range that equal the supplied value
	 *
	 * @param  Input    Any iterable type
	 * @param  InValue  Value to compare against
	 */
	template <typename InT, typename ValueT>
	[[nodiscard]] SIZE_T Count(const InT& Input, const ValueT& InValue)
	{
		SIZE_T Result = 0;
		for (const auto& Value : Input)
		{
			if (Value == InValue)
			{
				++Result;
			}
		}
		return Result;
	}

	/**
	 * Counts elements of a range that match a given predicate
	 *
	 * @param  Input      Any iterable type
	 * @param  Predicate  Condition which returns true for elements that should be counted and false for elements that should be skipped
	 */
	template <typename InT, typename PredicateT>
	[[nodiscard]] SIZE_T CountIf(const InT& Input, PredicateT Predicate)
	{
		SIZE_T Result = 0;
		for (const auto& Value : Input)
		{
			if (Invoke(Predicate, Value))
			{
				++Result;
			}
		}
		return Result;
	}

	/**
	 * Counts elements of a range whose projection equals the supplied value
	 *
	 * @param  Input       Any iterable type
	 * @param  InValue     Value to compare against
	 * @param  Projection  The projection to apply to the element
	 */
	template <typename InT, typename ValueT, typename ProjectionT>
	[[nodiscard]] SIZE_T CountBy(const InT& Input, const ValueT& InValue, ProjectionT Proj)
	{
		SIZE_T Result = 0;
		for (const auto& Value : Input)
		{
			if (Invoke(Proj, Value) == InValue)
			{
				++Result;
			}
		}
		return Result;
	}
}
