// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Impl/RangePointerType.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/UnrealTemplate.h" // For MoveTemp

namespace AlgoImpl
{
	template <typename RangeType, typename ValueType, typename ProjectionType>
	constexpr typename TRangePointerType<typename TRemoveReference<RangeType>::Type>::Type FindBy(RangeType&& Range, const ValueType& Value, ProjectionType Proj)
	{
		for (auto&& Elem : Forward<RangeType>(Range))
		{
			if (Invoke(Proj, Elem) == Value)
			{
				return &Elem;
			}
		}

		return nullptr;
	}

	template <typename RangeType, typename PredicateType>
	constexpr typename TRangePointerType<typename TRemoveReference<RangeType>::Type>::Type FindByPredicate(RangeType&& Range, PredicateType Pred)
	{
		for (auto&& Elem : Forward<RangeType>(Range))
		{
			if (Invoke(Pred, Elem))
			{
				return &Elem;
			}
		}

		return nullptr;
	}
}

namespace Algo
{
	/**
	 * Returns a pointer to the first element in the range which is equal to the given value.
	 *
	 * @param  Range  The range to search.
	 * @param  Value  The value to search for.
	 *
	 * @return A pointer to the first element found, or nullptr if none was found.
	 */
	template <typename RangeType, typename ValueType>
	[[nodiscard]] UE_REWRITE constexpr auto Find(RangeType&& Range, const ValueType& Value)
		-> decltype(AlgoImpl::FindBy(Forward<RangeType>(Range), Value, FIdentityFunctor()))
	{
		return AlgoImpl::FindBy(Forward<RangeType>(Range), Value, FIdentityFunctor());
	}

	/**
	 * Returns a pointer to the first element in the range whose projection is equal to the given value.
	 *
	 * @param  Range  The range to search.
	 * @param  Value  The value to search for.
	 * @param  Proj   The projection to apply to the element.
	 *
	 * @return A pointer to the first element found, or nullptr if none was found.
	 */
	template <typename RangeType, typename ValueType, typename ProjectionType>
	[[nodiscard]] UE_REWRITE constexpr auto FindBy(RangeType&& Range, const ValueType& Value, ProjectionType Proj)
		-> decltype(AlgoImpl::FindBy(Forward<RangeType>(Range), Value, MoveTemp(Proj)))
	{
		return AlgoImpl::FindBy(Forward<RangeType>(Range), Value, MoveTemp(Proj));
	}

	/**
	 * Returns a pointer to the first element in the range which matches the predicate.
	 *
	 * @param  Range  The range to search.
	 * @param  Pred   The predicate to search for.
	 *
	 * @return A pointer to the first element found, or nullptr if none was found.
	 */
	template <typename RangeType, typename PredicateType>
	[[nodiscard]] UE_REWRITE constexpr auto FindByPredicate(RangeType&& Range, PredicateType Pred)
		-> decltype(AlgoImpl::FindByPredicate(Forward<RangeType>(Range), MoveTemp(Pred)))
	{
		return AlgoImpl::FindByPredicate(Forward<RangeType>(Range), MoveTemp(Pred));
	}
}
