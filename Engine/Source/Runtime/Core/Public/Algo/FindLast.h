// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/UnrealTemplate.h" // For MoveTemp

namespace AlgoImpl
{
	template <typename T, typename ValueType, typename ProjectionType>
	T* FindLastBy(T* First, SIZE_T Num, const ValueType& Value, ProjectionType Proj)
	{
		for (T* Last = First + Num; First != Last;)
		{
			if (Invoke(Proj, *--Last) == Value)
			{
				return Last;
			}
		}

		return nullptr;
	}

	template <typename T, typename PredicateType>
	T* FindLastByPredicate(T* First, SIZE_T Num, PredicateType Pred)
	{
		for (T* Last = First + Num; First != Last;)
		{
			if (Invoke(Pred, *--Last))
			{
				return Last;
			}
		}

		return nullptr;
	}
}

namespace Algo
{
	/**
	 * Returns a pointer to the last element in the range which is equal to the given value.
	 *
	 * @param  Range  The range to search.
	 * @param  Value  The value to search for.
	 *
	 * @return A pointer to the last element found, or nullptr if none was found.
	 */
	template <typename RangeType, typename ValueType>
	[[nodiscard]] UE_REWRITE auto FindLast(RangeType&& Range, const ValueType& Value)
		-> decltype(AlgoImpl::FindLastBy(GetData(Range), GetNum(Range), Value, FIdentityFunctor()))
	{
		return AlgoImpl::FindLastBy(GetData(Range), GetNum(Range), Value, FIdentityFunctor());
	}

	/**
	 * Returns a pointer to the last element in the range whose projection is equal to the given value.
	 *
	 * @param  Range  The range to search.
	 * @param  Value  The value to search for.
	 * @param  Proj   The projection to apply to the element.
	 *
	 * @return A pointer to the last element found, or nullptr if none was found.
	 */
	template <typename RangeType, typename ValueType, typename ProjectionType>
	[[nodiscard]] UE_REWRITE auto FindLastBy(RangeType&& Range, const ValueType& Value, ProjectionType Proj)
		-> decltype(AlgoImpl::FindLastBy(GetData(Range), GetNum(Range), Value, MoveTemp(Proj)))
	{
		return AlgoImpl::FindLastBy(GetData(Range), GetNum(Range), Value, MoveTemp(Proj));
	}

	/**
	 * Returns a pointer to the last element in the range which matches the predicate.
	 *
	 * @param  Range  The range to search.
	 * @param  Pred   The predicate to search for.
	 *
	 * @return A pointer to the last element found, or nullptr if none was found.
	 */
	template <typename RangeType, typename PredicateType>
	[[nodiscard]] UE_REWRITE auto FindLastByPredicate(RangeType&& Range, PredicateType Pred)
		-> decltype(AlgoImpl::FindLastByPredicate(GetData(Range), GetNum(Range), MoveTemp(Pred)))
	{
		return AlgoImpl::FindLastByPredicate(GetData(Range), GetNum(Range), MoveTemp(Pred));
	}
}
