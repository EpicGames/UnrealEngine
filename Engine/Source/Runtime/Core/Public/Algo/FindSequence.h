// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Impl/RangePointerType.h"

namespace AlgoImpl
{
	template<typename WhereType, typename WhatType>
	constexpr WhereType* FindSequence(WhereType* First, WhereType* Last, WhatType* WhatFirst, WhatType* WhatLast)
	{
		for (; ; ++First)
		{
			WhereType* It = First;
			for (WhatType* WhatIt = WhatFirst; ; ++It, ++WhatIt)
			{
				if (WhatIt == WhatLast)
				{
					return First;
				}
				if (It == Last)
				{
					return nullptr;
				}
				if (!(*It == *WhatIt))
				{
					break;
				}
			}
		}
	}
}

namespace Algo
{

	/*
	* Searches for the first occurrence of a sequence of elements in another sequence.
	*
	* @param Where The range to search
	* @param What The sequence to search for.
	*
	* @return A pointer to the first occurrence of the "What" sequence in "Where" sequence, or nullptr if not found.
	*/
	template<typename RangeWhereType, typename RangeWhatType>
	[[nodiscard]] auto FindSequence(const RangeWhereType& Where, const RangeWhatType& What)
		-> decltype( AlgoImpl::FindSequence( GetData(Where), GetData(Where) + GetNum(Where), GetData(What), GetData(What) + GetNum(What)) )
	{
		if (GetNum(What) > GetNum(Where))
		{
			return nullptr;
		}
		else
		{
			return AlgoImpl::FindSequence(
				GetData(Where), GetData(Where) + GetNum(Where),
				GetData(What), GetData(What) + GetNum(What));
		}
	}

}
