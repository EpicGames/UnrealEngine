// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace Algo
{
	/**
	 * Finds a string in an array of sorted strings, by case-insensitive search, by using binary subdivision of the array.
	 *
	 * @param  Str          The string to look for.
	 * @param  SortedArray  The array of strings to search.  The strings must be sorted lexicographically, case-insensitively.
	 * @param  ArrayCount   The number of strings in the array.
	 *
	 * @return The index of the found string in the array, or -1 if the string was not found.
	 */
	[[nodiscard]] CORE_API int32 FindSortedStringCaseInsensitive(const TCHAR* Str, const TCHAR* const* SortedArray, int32 ArrayCount);

	/**
	 * Finds a string in an array of sorted strings, by case-insensitive search, by using binary subdivision of the array.
	 *
	 * @param  Str          The string to look for.
	 * @param  SortedArray  The array of strings to search.  The strings must be sorted lexicographically, case-insensitively.
	 *
	 * @return The index of the found string in the array, or -1 if the string was not found.
	 */
	template <int32 ArraySize>
	[[nodiscard]] UE_REWRITE int32 FindSortedStringCaseInsensitive(const TCHAR* Str, const TCHAR* const (&SortedArray)[ArraySize])
	{
		return FindSortedStringCaseInsensitive(Str, SortedArray, ArraySize);
	}
}
