// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Invoke.h"

namespace Algo
{
	/**
	 * Conditionally invokes a callable to each element in a range
	 *
	 * @param  Input      Any iterable type
	 * @param  Predicate  Condition which returns true for elements that should be called with and false for elements that should be skipped
	 * @param  Callable   Callable object
	 */
	template <typename InT, typename PredicateT, typename CallableT>
	void ForEachIf(InT&& Input, PredicateT Predicate, CallableT Callable)
	{
		for (auto& Value : Input)
		{
			if (Invoke(Predicate, Value))
			{
				Invoke(Callable, Value);
			}
		}
	}

	/**
	 * Invokes a callable to each element in a range
	 *
	 * @param  Input      Any iterable type
	 * @param  Callable   Callable object
	 */
	template <typename InT, typename CallableT>
	void ForEach(InT&& Input, CallableT Callable)
	{
		for (auto& Value : Input)
		{
			Invoke(Callable, Value);
		}
	}
}
