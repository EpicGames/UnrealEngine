// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Impl/BinaryHeap.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Less.h"
#include "Templates/UnrealTemplate.h" // For GetData, GetNum

namespace Algo
{
	/**
	 * Performs heap sort on the elements. Assumes < operator is defined for the element type.
	 *
	 * @param Range		The range to sort.
	 */
	template <typename RangeType>
	UE_REWRITE void HeapSort(RangeType&& Range)
	{
		AlgoImpl::HeapSortInternal(GetData(Range), GetNum(Range), FIdentityFunctor(), TLess<>());
	}

	/**
	 * Performs heap sort on the elements.
	 *
	 * @param Range		The range to sort.
	 * @param Predicate	A binary predicate object used to specify if one element should precede another.
	 */
	template <typename RangeType, typename PredicateType>
	UE_REWRITE void HeapSort(RangeType&& Range, PredicateType Predicate)
	{
		AlgoImpl::HeapSortInternal(GetData(Range), GetNum(Range), FIdentityFunctor(), MoveTemp(Predicate));
	}

	/**
	 * Performs heap sort on the elements. Assumes < operator is defined for the projected element type.
	 *
	 * @param Range		The range to sort.
	 * @param Projection	The projection to sort by when applied to the element.
	 */
	template <typename RangeType, typename ProjectionType>
	UE_REWRITE void HeapSortBy(RangeType&& Range, ProjectionType Projection)
	{
		AlgoImpl::HeapSortInternal(GetData(Range), GetNum(Range), Projection, TLess<>());
	}

	/**
	 * Performs heap sort on the elements.
	 *
	 * @param Range		The range to sort.
	 * @param Projection	The projection to sort by when applied to the element.
	 * @param Predicate	A binary predicate object, applied to the projection, used to specify if one element should precede another.
	 */
	template <typename RangeType, typename ProjectionType, typename PredicateType>
	UE_REWRITE void HeapSortBy(RangeType&& Range, ProjectionType Projection, PredicateType Predicate)
	{
		AlgoImpl::HeapSortInternal(GetData(Range), GetNum(Range), MoveTemp(Projection), MoveTemp(Predicate));
	}
}