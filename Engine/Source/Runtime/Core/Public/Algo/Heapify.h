// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Impl/BinaryHeap.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/Less.h"
#include "Templates/UnrealTemplate.h" // For GetData, GetNum

namespace Algo
{
	/**
	 * Builds an implicit min-heap from a range of elements. Assumes < operator is defined
	 * for the element type.
	 *
	 * @param Range	The range to heapify.
	 */
	template <typename RangeType>
	UE_REWRITE void Heapify(RangeType&& Range)
	{
		AlgoImpl::HeapifyInternal(GetData(Range), GetNum(Range), FIdentityFunctor(), TLess<>());
	}

	/** 
	 * Builds an implicit min-heap from a range of elements.
	 *
	 * @param Range		The range to heapify.
	 * @param Predicate	A binary predicate object used to specify if one element should precede another.
	 */
	template <typename RangeType, typename PredicateType>
	UE_REWRITE void Heapify(RangeType&& Range, PredicateType Predicate)
	{
		AlgoImpl::HeapifyInternal(GetData(Range), GetNum(Range), FIdentityFunctor(), Predicate);
	}
	
	/** 
	 * Builds an implicit min-heap from a range of elements. Assumes < operator is defined
	 * for the projected element type.
	 *
	 * @param Range			The range to heapify.
	 * @param Projection	The projection to apply to the elements.
	 */
	template <typename RangeType, typename ProjectionType, typename PredicateType>
	UE_REWRITE void HeapifyBy(RangeType&& Range, ProjectionType Projection)
	{
		AlgoImpl::HeapifyInternal(GetData(Range), GetNum(Range), Projection, TLess<>());
	}

	/** 
	 * Builds an implicit min-heap from a range of elements.
	 *
	 * @param Range			The range to heapify.
	 * @param Projection	The projection to apply to the elements.
	 * @param Predicate		A binary predicate object used to specify if one element should precede another.
	 */
	template <typename RangeType, typename ProjectionType, typename PredicateType>
	UE_REWRITE void HeapifyBy(RangeType&& Range, ProjectionType Projection, PredicateType Predicate)
	{
		AlgoImpl::HeapifyInternal(GetData(Range), GetNum(Range), Projection, Predicate);
	}
}