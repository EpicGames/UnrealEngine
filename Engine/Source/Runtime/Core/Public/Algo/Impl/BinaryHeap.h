// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Invoke.h" // for older _MSC_FULL_VER only - see usage below
#include "Templates/Projection.h"
#include "Templates/ReversePredicate.h"

#include <type_traits>

// Workaround for a codegen bug that was discovered related to member function pointer projections that are virtual
#if defined(_MSC_FULL_VER) && _MSC_FULL_VER < 194134123
	#define UE_VIRTUAL_PROJECTION_WORKAROUND 1
#else
	#define UE_VIRTUAL_PROJECTION_WORKAROUND 0
#endif

namespace AlgoImpl
{
	/**
	 * Gets the index of the left child of node at Index.
	 *
	 * @param	Index Node for which the left child index is to be returned.
	 * @returns	Index of the left child.
	 */
	template <typename IndexType>
	FORCEINLINE IndexType HeapGetLeftChildIndex(IndexType Index)
	{
		return Index * 2 + 1;
	}

	/** 
	 * Checks if node located at Index is a leaf or not.
	 *
	 * @param	Index Node index.
	 * @returns	true if node is a leaf, false otherwise.
	 */
	template <typename IndexType>
	FORCEINLINE bool HeapIsLeaf(IndexType Index, IndexType Count)
	{
		return HeapGetLeftChildIndex(Index) >= Count;
	}

	/**
	 * Gets the parent index for node at Index.
	 *
	 * @param	Index node index.
	 * @returns	Parent index.
	 */
	template <typename IndexType>
	FORCEINLINE IndexType HeapGetParentIndex(IndexType Index)
	{
		return (Index - 1) / 2;
	}

	/**
	 * Fixes a possible violation of order property between node at Index and a child.
	 *
	 * @param	Heap		Random access iterator to the first element of a binary heap.
	 * @param	Index		Node index.
	 * @param	Count		Size of the heap.
	 * @param	InProj		The projection to apply to the elements.
	 * @param	Predicate	A binary predicate object used to specify if one element should precede another.
	 */
	template <typename IteratorType, typename IndexType, typename ProjectionType, typename PredicateType>
	inline void HeapSiftDown(IteratorType Heap, IndexType Index, const IndexType Count, const ProjectionType& InProj, const PredicateType& Predicate)
	{
#if UE_VIRTUAL_PROJECTION_WORKAROUND
		const ProjectionType& Proj = InProj;
#else
		auto&& Proj = Projection(InProj);
#endif
		while (!HeapIsLeaf(Index, Count))
		{
			const IndexType LeftChildIndex = HeapGetLeftChildIndex(Index);
			const IndexType RightChildIndex = LeftChildIndex + 1;

			IndexType MinChildIndex = LeftChildIndex;
			if (RightChildIndex < Count)
			{
#if UE_VIRTUAL_PROJECTION_WORKAROUND
				MinChildIndex = Predicate( Invoke(Proj, Heap[LeftChildIndex]), Invoke(Proj, Heap[RightChildIndex]) ) ? LeftChildIndex : RightChildIndex;
#else
				MinChildIndex = Predicate( Proj(Heap[LeftChildIndex]), Proj(Heap[RightChildIndex]) ) ? LeftChildIndex : RightChildIndex;
#endif
			}

#if UE_VIRTUAL_PROJECTION_WORKAROUND
			if (!Predicate( Invoke(Proj, Heap[MinChildIndex]), Invoke(Proj, Heap[Index]) ))
#else
			if (!Predicate( Proj(Heap[MinChildIndex]), Proj(Heap[Index]) ))
#endif
			{
				break;
			}

			Swap(Heap[Index], Heap[MinChildIndex]);
			Index = MinChildIndex;
		}
	}

	/**
	 * Fixes a possible violation of order property between node at NodeIndex and a parent.
	 *
	 * @param	Heap		Random access iterator to the first element of a binary heap.
	 * @param	RootIndex	How far to go up?
	 * @param	NodeIndex	Node index.
	 * @param	InProj		The projection to apply to the elements.
	 * @param	Predicate	A binary predicate object used to specify if one element should precede another.
	 *
	 * @return	The new index of the node that was at NodeIndex
	 */
	template <class IteratorType, typename IndexType, typename ProjectionType, class PredicateType>
	inline IndexType HeapSiftUp(IteratorType Heap, IndexType RootIndex, IndexType NodeIndex, const ProjectionType& InProj, const PredicateType& Predicate)
	{
#if UE_VIRTUAL_PROJECTION_WORKAROUND
		const ProjectionType& Proj = InProj;
#else
		auto&& Proj = Projection(InProj);
#endif
		while (NodeIndex > RootIndex)
		{
			IndexType ParentIndex = HeapGetParentIndex(NodeIndex);
#if UE_VIRTUAL_PROJECTION_WORKAROUND
			if (!Predicate( Invoke(Proj, Heap[NodeIndex]), Invoke(Proj, Heap[ParentIndex]) ))
#else
			if (!Predicate( Proj(Heap[NodeIndex]), Proj(Heap[ParentIndex]) ))
#endif
			{
				break;
			}

			Swap(Heap[NodeIndex], Heap[ParentIndex]);
			NodeIndex = ParentIndex;
		}

		return NodeIndex;
	}

	/** 
	 * Builds an implicit min-heap from a range of elements.
	 * This is the internal function used by Heapify overrides.
	 *
	 * @param	First		Random access iterator to the first element to heapify.
	 * @param	Num			The number of items to heapify.
	 * @param	Proj		The projection to apply to the elements.
	 * @param	Predicate	A binary predicate object used to specify if one element should precede another.
	 */
	template <typename IteratorType, typename IndexType, typename ProjectionType, typename PredicateType>
	inline void HeapifyInternal(IteratorType First, IndexType Num, ProjectionType Proj, PredicateType Predicate)
	{
		if constexpr (std::is_signed_v<IndexType>)
		{
			checkf(Num >= 0, TEXT("Algo::HeapifyInternal called with negative count"));
		}

		if (Num < 2)
		{
			return;
		}

		IndexType Index = HeapGetParentIndex(Num - 1);
		for (;;)
		{
			HeapSiftDown(First, Index, Num, Proj, Predicate);
			if (Index == 0)
			{
				return;
			}
			--Index;
		}
	}

	/**
	 * Performs heap sort on the elements.
	 * This is the internal sorting function used by HeapSort overrides.
	 *
	 * @param	First		Random access iterator to the first element to sort.
	 * @param	Num			The number of elements to sort.
	 * @param	Proj		The projection to apply to the elements.
	 * @param	Predicate	A binary predicate object used to specify if one element should precede another.
	 */
	template <typename IteratorType, typename IndexType, typename ProjectionType, class PredicateType>
	void HeapSortInternal(IteratorType First, IndexType Num, ProjectionType Proj, PredicateType Predicate)
	{
		if constexpr (std::is_signed_v<IndexType>)
		{
			checkf(Num >= 0, TEXT("Algo::HeapSortInternal called with negative count"));
		}

		if (Num < 2)
		{
			return;
		}

		TReversePredicate< PredicateType > ReversePredicateWrapper(Predicate); // Reverse the predicate to build a max-heap instead of a min-heap
		HeapifyInternal(First, Num, Proj, ReversePredicateWrapper);

		for (IndexType Index = Num - 1; Index > 0; Index--)
		{
			Swap(First[0], First[Index]);

			HeapSiftDown(First, (IndexType)0, Index, Proj, ReversePredicateWrapper);
		}
	}
}
