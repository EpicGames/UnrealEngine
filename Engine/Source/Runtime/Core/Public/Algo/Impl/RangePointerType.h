// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UnrealTemplate.h" // For DeclVal

namespace AlgoImpl
{
	/**
	 * Traits class whose Type member is the pointer type to an element of the range.
	 */
	template <typename RangeType>
	struct TRangePointerType
	{
		using Type = decltype(&*DeclVal<RangeType&>().begin());
	};

	template <typename T, unsigned int N>
	struct TRangePointerType<T[N]>
	{
		using Type = T*;
	};
}
