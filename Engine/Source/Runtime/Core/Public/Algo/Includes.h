// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Less.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/UnrealTemplate.h"

namespace Algo::Private
{
	template <typename DataTypeA, typename SizeTypeA, typename DataTypeB, typename SizeTypeB, typename ProjectionType, typename SortPredicateType>
	constexpr bool Includes(const DataTypeA* DataA, SizeTypeA NumA, const DataTypeB* DataB, SizeTypeB NumB, ProjectionType Projection, SortPredicateType SortPredicate)
	{
		SizeTypeA IndexA = 0;
		SizeTypeB IndexB = 0;

		for (; IndexB < NumB; ++IndexA)
		{
			if (IndexA >= NumA)
			{
				return false;
			}

			const auto& RefA = Invoke(Projection, DataA[IndexA]);
			const auto& RefB = Invoke(Projection, DataB[IndexB]);

			if (Invoke(SortPredicate, RefB, RefA))
			{
				return false;
			}

			if (!Invoke(SortPredicate, RefA, RefB))
			{
				++IndexB;
			}
		}
		return true;
	}
}

namespace Algo
{
	/**
	 * Checks if one sorted contiguous container is a subsequence of another sorted contiguous container by comparing pairs of elements.
	 * The subsequence does not need to be contiguous. Uses operator< to compare pairs of elements.
	 *
	 * @param  RangeA        Container of elements to search through. Must be already sorted by operator<.
	 * @param  RangeB        Container of elements to search for. Must be already sorted by operator<.
	 *
	 * @return True if RangeB is a subsequence of RangeA, false otherwise.
	 */
	template <typename RangeTypeA, typename RangeTypeB>
	[[nodiscard]] constexpr bool Includes(const RangeTypeA& RangeA, const RangeTypeB& RangeB)
	{
		return Private::Includes(GetData(RangeA), GetNum(RangeA), GetData(RangeB), GetNum(RangeB), FIdentityFunctor(), TLess<>());
	}
	
	/**
	 * Checks if one sorted contiguous container is a subsequence of another sorted contiguous container by comparing pairs of elements using a custom predicate.
	 * The subsequence does not need to be contiguous.
	 *
	 * @param  RangeA        Container of elements to search through. Must be already sorted by SortPredicate.
	 * @param  RangeB        Container of elements to search for. Must be already sorted by SortPredicate.
	 * @param  SortPredicate A binary predicate object used to specify if one element should precede another.
	 *
	 * @return True if RangeB is a subsequence of RangeA according to the provided predicate, false otherwise.
	 */
	template <typename RangeTypeA, typename RangeTypeB, typename SortPredicateType>
	[[nodiscard]] constexpr bool Includes(const RangeTypeA& RangeA, const RangeTypeB& RangeB, SortPredicateType SortPredicate)
	{
		return Private::Includes(GetData(RangeA), GetNum(RangeA), GetData(RangeB), GetNum(RangeB), FIdentityFunctor(), MoveTemp(SortPredicate));
	}
	
	/**
	 * Checks if one sorted contiguous container is a subsequence of another sorted contiguous container by comparing pairs of projected elements.
	 * The subsequence does not need to be contiguous. Uses operator< to compare pairs of projected elements.
	 *
	 * @param  RangeA        Container of elements to search through. Must be already sorted by operator<.
	 * @param  RangeB        Container of elements to search for. Must be already sorted by operator<.
	 * @param  Projection    Projection to apply to the elements before comparing them.
	 *
	 * @return True if RangeB is a subsequence of RangeA, based on the comparison of projected elements, false otherwise.
	 */
	template <typename RangeTypeA, typename RangeTypeB, typename ProjectionType>
	[[nodiscard]] constexpr bool IncludesBy(const RangeTypeA& RangeA, const RangeTypeB& RangeB, ProjectionType Projection)
	{
		return Private::Includes(GetData(RangeA), GetNum(RangeA), GetData(RangeB), GetNum(RangeB), MoveTemp(Projection), TLess<>());
	}
	
	/**
	 * Checks if one sorted contiguous container is a subsequence of another sorted contiguous container by comparing pairs of projected elements using a custom predicate.
	 * The subsequence does not need to be contiguous.
	 *
	 * @param  RangeA        Container of elements to search through. Must be already sorted by SortPredicate.
	 * @param  RangeB        Container of elements to search for. Must be already sorted by SortPredicate.
	 * @param  Projection    Projection to apply to the elements before comparing them.
	 * @param  SortPredicate A binary predicate object, applied to the projection, used to specify if one element should precede another.
	 *
	 * @return True if RangeB is a subsequence of RangeA according to the comparison of projected elements using the provided predicate, false otherwise.
	 */
	template <typename RangeTypeA, typename RangeTypeB, typename ProjectionType, typename SortPredicateType>
	[[nodiscard]] constexpr bool IncludesBy(const RangeTypeA& RangeA, const RangeTypeB& RangeB, ProjectionType Projection, SortPredicateType SortPredicate)
	{
		return Private::Includes(GetData(RangeA), GetNum(RangeA), GetData(RangeB), GetNum(RangeB), MoveTemp(Projection), MoveTemp(SortPredicate));
	}
}
