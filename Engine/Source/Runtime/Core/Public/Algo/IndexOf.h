// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/UnrealTemplate.h"

namespace AlgoImpl
{
	template <typename RangeType, typename ValueType, typename ProjectionType>
	auto IndexOfBy(const RangeType& Range, const ValueType& Value, ProjectionType Proj)
	{
		auto Num = GetNum(Range);
		auto Data = GetData(Range);

		using SizeType = decltype(Num);
		for (SizeType Index = 0; Index < Num; ++Index)
		{
			if (Invoke(Proj, Data[Index]) == Value)
			{
				return Index;
			}
		}

		return (SizeType)-1;
	}

	template <typename RangeType, typename PredicateType>
	auto IndexOfByPredicate(const RangeType& Range, PredicateType Pred)
	{
		auto Num = GetNum(Range);
		auto Data = GetData(Range);

		using SizeType = decltype(Num);
		for (SizeType Index = 0; Index < Num; ++Index)
		{
			if (Invoke(Pred, Data[Index]))
			{
				return Index;
			}
		}

		return (SizeType)-1;
	}
}

namespace Algo
{
	/**
	 * Returns the index of the first element in the range which is equal to the given value.
	 *
	 * @param  Range  The range to search.
	 * @param  Value  The value to search for.
	 *
	 * @return The index of the first element found, or (SizeType)-1 if none was found (where SizeType = decltype(GetNum(Range)))
	 */
	template <typename RangeType, typename ValueType>
	[[nodiscard]] UE_REWRITE auto IndexOf(const RangeType& Range, const ValueType& Value)
	{
		static_assert(TIsContiguousContainer<RangeType>::Value, "Must only be used with contiguous containers");
		return AlgoImpl::IndexOfBy(Range, Value, FIdentityFunctor());
	}

	/**
	 * Returns the index of the first element in the range whose projection is equal to the given value.
	 *
	 * @param  Range  The range to search.
	 * @param  Value  The value to search for.
	 * @param  Proj   The projection to apply to the element.
	 *
	 * @return The index of the first element found, or (SizeType)-1 if none was found (where SizeType = decltype(GetNum(Range)))
	 */
	template <typename RangeType, typename ValueType, typename ProjectionType>
	[[nodiscard]] UE_REWRITE auto IndexOfBy(const RangeType& Range, const ValueType& Value, ProjectionType Proj)
	{
		static_assert(TIsContiguousContainer<RangeType>::Value, "Must only be used with contiguous containers");
		return AlgoImpl::IndexOfBy(Range, Value, MoveTemp(Proj));
	}

	/**
	 * Returns the index of the first element in the range which matches the predicate.
	 *
	 * @param  Range  The range to search.
	 * @param  Pred   The predicate to search for.
	 *
	 * @return The index of the first element found, or (SizeType)-1 if none was found (where SizeType = decltype(GetNum(Range)))
	 */
	template <typename RangeType, typename PredicateType>
	[[nodiscard]] UE_REWRITE auto IndexOfByPredicate(const RangeType& Range, PredicateType Pred)
	{
		static_assert(TIsContiguousContainer<RangeType>::Value, "Must only be used with contiguous containers");
		return AlgoImpl::IndexOfByPredicate(Range, MoveTemp(Pred));
	}
}
