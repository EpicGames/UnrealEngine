// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Impl/BinaryHeap.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h" // for older _MSC_FULL_VER only - see usage below
#include "Templates/Projection.h"
#include "Templates/Less.h"
#include "Templates/UnrealTemplate.h" // For GetData, GetNum

#include <iterator>

#ifndef UE_VIRTUAL_PROJECTION_WORKAROUND
	#error "Expected UE_VIRTUAL_PROJECTION_WORKAROUND from BinaryHeap.h to be defined."
#endif

namespace AlgoImpl
{
	/**
	 * Implementation of an introspective sort. Starts with quick sort and switches to heap sort when the iteration depth is too big.
	 * The sort is unstable, meaning that the ordering of equal items is not necessarily preserved.
	 * This is the internal sorting function used by IntroSort overrides.
	 *
	 * @param First			Random access iterator to the first element to sort.
	 * @param Num			The number of items to sort.
	 * @param Proj			The projection to sort by when applied to the element.
	 * @param Predicate		predicate class
	 */
	template <typename IteratorType, typename IndexType, typename ProjectionType, typename PredicateType>
	void IntroSortInternal(IteratorType First, IndexType Num, ProjectionType Proj, PredicateType Predicate)
	{
		struct FStack
		{
			IteratorType Min;
			IteratorType Max;
			uint32 MaxDepth;
		};

		if( Num < 2 )
		{
			return;
		}

		FStack RecursionStack[32] = { {First, First + (Num - 1), (uint32)(FMath::Loge((float)Num) * 2.f)} };
		FStack Current;
		FStack Inner;
		for( FStack* StackTop=RecursionStack; StackTop>=RecursionStack; --StackTop ) //-V625
		{
			Current = *StackTop;

		Loop:
			IndexType Count = (IndexType)(Current.Max - Current.Min + 1);

			if ( Current.MaxDepth == 0 )
			{
				// We're too deep into quick sort, switch to heap sort
				HeapSortInternal( Current.Min, Count, Proj, Predicate );
				continue;
			}

			if( Count <= 8 )
			{
				// Use simple selection sort.
				while( Current.Max > Current.Min )
				{
					IteratorType Max;
					IteratorType Item;
					for( Max=Current.Min, Item=Current.Min+1; Item<=Current.Max; ++Item )
					{
#if UE_VIRTUAL_PROJECTION_WORKAROUND
						if( Invoke( Predicate, Invoke( Proj, *Max ), Invoke( Proj, *Item ) ) )
#else
						if( Predicate( Proj( *Max ), Proj( *Item ) ) )
#endif
						{
							Max = Item;
						}
					}
					Swap( *Max, *Current.Max-- );
				}
			}
			else
			{
				// Grab middle element so sort doesn't exhibit worst-cast behavior with presorted lists.
				Swap( Current.Min[Count/2], Current.Min[0] );

				// Divide list into two halves, one with items <=Current.Min, the other with items >Current.Max.
				Inner.Min = Current.Min;
				Inner.Max = Current.Max+1;
				for( ; ; )
				{
#if UE_VIRTUAL_PROJECTION_WORKAROUND
					while( ++Inner.Min<=Current.Max && !Invoke( Predicate, Invoke( Proj, *Current.Min ), Invoke( Proj, *Inner.Min ) ) );
					while( --Inner.Max> Current.Min && !Invoke( Predicate, Invoke( Proj, *Inner.Max ), Invoke( Proj, *Current.Min ) ) );
#else
					while( ++Inner.Min<=Current.Max && !Predicate( Proj( *Current.Min ), Proj( *Inner.Min ) ) );
					while( --Inner.Max> Current.Min && !Predicate( Proj( *Inner.Max ), Proj( *Current.Min ) ) );
#endif
					if( Inner.Min>Inner.Max )
					{
						break;
					}
					Swap( *Inner.Min, *Inner.Max );
				}
				Swap( *Current.Min, *Inner.Max );

				--Current.MaxDepth;

				// Save big half and recurse with small half.
				if( Inner.Max-1-Current.Min >= Current.Max-Inner.Min )
				{
					if( Current.Min+1 < Inner.Max )
					{
						StackTop->Min = Current.Min;
						StackTop->Max = Inner.Max - 1;
						StackTop->MaxDepth = Current.MaxDepth;
						StackTop++;
					}
					if( Current.Max>Inner.Min )
					{
						Current.Min = Inner.Min;
						goto Loop;
					}
				}
				else
				{
					if( Current.Max>Inner.Min )
					{
						StackTop->Min = Inner  .Min;
						StackTop->Max = Current.Max;
						StackTop->MaxDepth = Current.MaxDepth;
						StackTop++;
					}
					if( Current.Min+1<Inner.Max )
					{
						Current.Max = Inner.Max - 1;
						goto Loop;
					}
				}
			}
		}
	}
}

namespace Algo
{
	/**
	 * Sort a range of elements using its operator<. The sort is unstable.
	 *
	 * @param Range	The range to sort.
	 */
	template <typename RangeType>
	UE_REWRITE void IntroSort(RangeType&& Range)
	{
		AlgoImpl::IntroSortInternal(std::begin(Range), std::end(Range) - std::begin(Range), FIdentityFunctor(), TLess<>());
	}

	/**
	 * Sort a range of elements using a user-defined predicate class. The sort is unstable.
	 *
	 * @param Range		The range to sort.
	 * @param Predicate	A binary predicate object used to specify if one element should precede another.
	 */
	template <typename RangeType, typename PredicateType>
	UE_REWRITE void IntroSort(RangeType&& Range, PredicateType Predicate)
	{
		AlgoImpl::IntroSortInternal(std::begin(Range), std::end(Range) - std::begin(Range), FIdentityFunctor(), MoveTemp(Predicate));
	}

	/**
	 * Sort a range of elements by a projection using the projection's operator<. The sort is unstable.
	 *
	 * @param Range			The range to sort.
	 * @param Proj			The projection to sort by when applied to the element.
	 */
	template <typename RangeType, typename ProjectionType>
	UE_REWRITE void IntroSortBy(RangeType&& Range, ProjectionType Proj)
	{
#if UE_VIRTUAL_PROJECTION_WORKAROUND
		AlgoImpl::IntroSortInternal(std::begin(Range), std::end(Range) - std::begin(Range), MoveTemp(Proj), TLess<>());
#else
		AlgoImpl::IntroSortInternal(std::begin(Range), std::end(Range) - std::begin(Range), Projection(MoveTemp(Proj)), TLess<>());
#endif
	}

	/**
	 * Sort a range of elements by a projection using a user-defined predicate class. The sort is unstable.
	 *
	 * @param Range			The range to sort.
	 * @param Proj			The projection to sort by when applied to the element.
	 * @param Predicate		A binary predicate object, applied to the projection, used to specify if one element should precede another.
	 */
	template <typename RangeType, typename ProjectionType, typename PredicateType>
	UE_REWRITE void IntroSortBy(RangeType&& Range, ProjectionType Proj, PredicateType Predicate)
	{
#if UE_VIRTUAL_PROJECTION_WORKAROUND
		AlgoImpl::IntroSortInternal(std::begin(Range), std::end(Range) - std::begin(Range), MoveTemp(Proj), MoveTemp(Predicate));
#else
		AlgoImpl::IntroSortInternal(std::begin(Range), std::end(Range) - std::begin(Range), Projection(MoveTemp(Proj)), Projection(MoveTemp(Predicate)));
#endif
	}
}
