// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Impl/BinaryHeap.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/Less.h"
#include "Templates/UnrealTemplate.h" // For GetData, GetNum

namespace AlgoImpl
{
	/**
	 * Verifies that the range is a min-heap (parent <= child)
	 * This is the internal function used by IsHeap overrides.
	 *
	 * @param	Heap		Pointer to the first element of a binary heap.
	 * @param	Num			the number of items in the heap.
	 * @param	Projection	The projection to apply to the elements.
	 * @param	Predicate	A binary predicate object used to specify if one element should precede another.
	 *
	 * @return	returns		true if the range is a min-heap
	 */
	template <typename RangeValueType, typename IndexType, typename ProjectionType, typename PredicateType>
	bool IsHeapInternal(const RangeValueType* Heap, IndexType Num, ProjectionType Projection, PredicateType Predicate)
	{
		for (IndexType Index = 1; Index < Num; Index++)
		{
			IndexType ParentIndex = HeapGetParentIndex(Index);
			if (Predicate( Invoke(Projection, Heap[Index]), Invoke(Projection, Heap[ParentIndex]) ))
			{
				return false;
			}
		}

		return true;
	}
}

namespace Algo
{
	/**
	 * Verifies that the range is a min-heap (parent <= child). Assumes < operator is defined for the element type.
	 *
	 * @param	Range	The range to verify.
	 *
	 * @return	returns	true if the range is a min-heap
	 */
	template <typename RangeType>
	[[nodiscard]] UE_REWRITE bool IsHeap(const RangeType& Range)
	{
		return AlgoImpl::IsHeapInternal(GetData(Range), GetNum(Range), FIdentityFunctor(), TLess<>());
	}

	/**
	 * Verifies that the range is a min-heap (parent <= child)
	 *
	 * @param	Range		The range to verify.
	 * @param	Predicate	A binary predicate object used to specify if one element should precede another.
	 *
	 * @return	returns		true if the range is a min-heap
	 */
	template <typename RangeType, typename PredicateType>
	[[nodiscard]] UE_REWRITE bool IsHeap(const RangeType& Range, PredicateType Predicate)
	{
		return AlgoImpl::IsHeapInternal(GetData(Range), GetNum(Range), FIdentityFunctor(), MoveTemp(Predicate));
	}

	/**
	 * Verifies that the range is a min-heap (parent <= child). Assumes < operator is defined for the projected element type.
	 *
	 * @param	Range		The range to verify.
	 * @param	Projection	The projection to apply to the elements.
	 *
	 * @return	returns		true if the range is a min-heap
	 */
	template <typename RangeType, typename ProjectionType>
	[[nodiscard]] UE_REWRITE bool IsHeapBy(const RangeType& Range, ProjectionType Projection)
	{
		return AlgoImpl::IsHeapInternal(GetData(Range), GetNum(Range), MoveTemp(Projection), TLess<>());
	}

	/**
	 * Verifies that the range is a min-heap (parent <= child)
	 *
	 * @param	Range		The range to verify.
	 * @param	Projection	The projection to apply to the elements.
	 * @param	Predicate	A binary predicate object used to specify if one element should precede another.
	 *
	 * @return	returns		true if the range is a min-heap
	 */
	template <typename RangeType, typename ProjectionType, typename PredicateType>
	[[nodiscard]] UE_REWRITE bool IsHeapBy(const RangeType& Range, ProjectionType Projection, PredicateType Predicate)
	{
		return AlgoImpl::IsHeapInternal(GetData(Range), GetNum(Range), MoveTemp(Projection), MoveTemp(Predicate));
	}
}