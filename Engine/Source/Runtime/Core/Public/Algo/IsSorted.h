// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/Less.h"
#include "Templates/UnrealTemplate.h" // For GetData, GetNum, MoveTemp


namespace AlgoImpl
{
	template <typename T, typename IndexType, typename ProjectionType, typename PredType>
	bool IsSortedBy(const T* Range, IndexType RangeSize, ProjectionType Proj, PredType Pred)
	{
		if (RangeSize == 0)
		{
			return true;
		}

		// When comparing N elements, we do N-1 comparisons
		--RangeSize;

		const T* Next = Range + 1;
		for (;;)
		{
			if (RangeSize == 0)
			{
				return true;
			}

			auto&& Ref1 = Invoke(Proj, *Next);
			auto&& Ref2 = Invoke(Proj, *Range);
			if (Invoke(Pred, Ref1, Ref2))
			{
				return false;
			}

			++Range;
			++Next;
			--RangeSize;
		}
	}

	template <typename T>
	struct TLess
	{
		FORCEINLINE bool operator()(const T& Lhs, const T& Rhs) const
		{
			return Lhs < Rhs;
		}
	};
}

namespace Algo
{
	/**
	 * Tests if a range is sorted by its element type's operator<.
	 *
	 * @param  Range  The container to test for being sorted.
	 *
	 * @return true if the range is sorted, false otherwise.
	 */
	template <typename RangeType>
	[[nodiscard]] UE_REWRITE bool IsSorted(const RangeType& Range)
	{
		return AlgoImpl::IsSortedBy(GetData(Range), GetNum(Range), FIdentityFunctor(), TLess<>());
	}

	/**
	 * Tests if a range is sorted by a user-defined predicate.
	 *
	 * @param  Range  The container to test for being sorted.
	 * @param  Pred   A binary sorting predicate which describes the ordering of the elements in the array.
	 *
	 * @return true if the range is sorted, false otherwise.
	 */
	template <typename RangeType, typename PredType>
	[[nodiscard]] UE_REWRITE bool IsSorted(const RangeType& Range, PredType Pred)
	{
		return AlgoImpl::IsSortedBy(GetData(Range), GetNum(Range), FIdentityFunctor(), MoveTemp(Pred));
	}

	/**
	 * Tests if a range is sorted by a projection of the element type, using the projection's operator<.
	 *
	 * @param  Range  The container to test for being sorted.
	 *
	 * @return true if the range is sorted, false otherwise.
	 */
	template <typename RangeType, typename ProjectionType>
	[[nodiscard]] UE_REWRITE bool IsSortedBy(const RangeType& Range, ProjectionType Projection)
	{
		return AlgoImpl::IsSortedBy(GetData(Range), GetNum(Range), MoveTemp(Projection), TLess<>());
	}

	/**
	 * Tests if a range is sorted by a projection of the element type, using a user-defined predicate on the projection.
	 *
	 * @param  Range  The container to test for being sorted.
	 * @param  Pred   A binary sorting predicate which describes the ordering of the elements in the array.
	 *
	 * @return true if the range is sorted, false otherwise.
	 */
	template <typename RangeType, typename ProjectionType, typename PredType>
	[[nodiscard]] UE_REWRITE bool IsSortedBy(const RangeType& Range, ProjectionType Projection, PredType Pred)
	{
		return AlgoImpl::IsSortedBy(GetData(Range), GetNum(Range), MoveTemp(Projection), MoveTemp(Pred));
	}
}
