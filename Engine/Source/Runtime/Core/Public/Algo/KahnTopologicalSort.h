// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Invoke.h"
#include "Templates/UnrealTemplate.h"
#include "Traits/ElementType.h"

namespace AlgoImpl
{
	/**
	 * KahnTopologicalSort converts vertices and edges from ElementType to indexes of the vertex in the original
	 * UniqueRange of ElementType. FKahnHandle is how vertices are represented during the graph algorithm
	 */
	typedef int32 FKahnHandle;

	/** Scratch variables shared across multiple calls to FindMostIndependentMutuallyReachableVertexSet */
	struct FMutuallyReachableVertexSetContext
	{
		TSet<FKahnHandle> VisitedReferences;
		TSet<FKahnHandle> VisitedDependencies;
		TSet<FKahnHandle> MaximalMRVS;
		TSet<FKahnHandle> Culprits;
		TArray<FKahnHandle> Stack;
		TArray<FKahnHandle> Cycle;
	};
	/** Some variables shared with subfunctions */
	struct FKahnContext
	{
		TSet<FKahnHandle> RemainingVertices;
		TArray<TArray<FKahnHandle>> Referencers;
		TArray<TArray<FKahnHandle>> Dependencies;
		TArray<int32> DependencyCount;
		TOptional<FMutuallyReachableVertexSetContext> MRVSContext;
	};

	// Helper functions
	template <typename RangeType, typename GetElementDependenciesType>
	void KahnTopologicalSort_CreateWorkingGraph(FKahnContext& Context, RangeType& UniqueRange,
		GetElementDependenciesType GetElementDependencies, TSet<FKahnHandle>& OutInitialIndependents);
	const TSet<FKahnHandle>& FindMostIndependentMutuallyReachableVertexSet(FKahnContext& Context);
}

namespace Algo
{
	/** Flags for behavior of TopologicalSort; see the function comment in TopologicalSort.h */
	enum class ETopologicalSort
	{
		None,
		AllowCycles,
	};
	ENUM_CLASS_FLAGS(ETopologicalSort);

	/** Public entrypoint. Implements Algo::TopologicalSort using the Kahn Topological Sort algorithm. */
	template <typename RangeType, typename GetElementDependenciesType>
	bool KahnTopologicalSort(RangeType&& UniqueRange, GetElementDependenciesType GetElementDependencies,
		ETopologicalSort Flags)
	{
		using namespace AlgoImpl;
		using ElementType = typename TElementType<RangeType>::Type;

		FKahnContext Context;
		TSet<FKahnHandle> IndependentVertices;
		KahnTopologicalSort_CreateWorkingGraph(Context, UniqueRange,
			Forward<GetElementDependenciesType>(GetElementDependencies), IndependentVertices);

		// Initialize the graph search
		TArray<TArray<FKahnHandle>>& Referencers(Context.Referencers);
		TArray<TArray<FKahnHandle>>& Dependencies(Context.Dependencies);
		TArray<int32>& DependencyCount(Context.DependencyCount);
		TSet<FKahnHandle>& RemainingVertices(Context.RemainingVertices);
		TSet<FKahnHandle> NewIndependentVertices;
		TArray<FKahnHandle> SortedRange;
		int32 NumElements = Dependencies.Num();
		SortedRange.Reserve(NumElements);
		RemainingVertices.Reserve(NumElements);
		for (FKahnHandle Vertex = 0; Vertex < NumElements; ++Vertex)
		{
			RemainingVertices.Add(Vertex);
		}

		// Sort graph so that vertices with no dependencies (leaves) always go first.
		while (RemainingVertices.Num() > 0)
		{
			if (IndependentVertices.Num() == 0)
			{
				// If there are no independent vertices then there is a cycle in the graph
				if (!EnumHasAnyFlags(Flags, ETopologicalSort::AllowCycles))
				{
					return false;
				}

				// When all remaining vertices are in cycles, find the maximal MutuallyReachableVertexSet
				//  (a cycle is an MRVS) that is independent of all the other MRVS's.
				const TSet<FKahnHandle>& MRVS = FindMostIndependentMutuallyReachableVertexSet(Context);
				check(!MRVS.IsEmpty());
				for (FKahnHandle Vertex : MRVS)
				{
					// Add all the vertices in the MutuallyReachableVertexSet to the SortedRange in arbitrary order
					IndependentVertices.Add(Vertex);
					// Mark that we should no longer consider any vertices in the MRVS when looking for NewIndependentVertices;
					// they have already been removed
					DependencyCount[Vertex] = INDEX_NONE;
				}
			}

			NewIndependentVertices.Reset();
			for (FKahnHandle Vertex : IndependentVertices)
			{
				for (FKahnHandle Referencer : Referencers[Vertex])
				{
					int32& ReferencerDependencyCount = DependencyCount[Referencer];
					if (ReferencerDependencyCount == INDEX_NONE)
					{
						// Don't read or write dependencycount for referencers we removed due to a cycle
						continue;
					}
					check(ReferencerDependencyCount > 0);
					if (--ReferencerDependencyCount == 0)
					{
						NewIndependentVertices.Add(Referencer);
					}
				}
#if DO_CHECK
				int32 RemainingNum = RemainingVertices.Num();
#endif
				RemainingVertices.Remove(Vertex);
#if DO_CHECK
				check(RemainingVertices.Num() == RemainingNum - 1); // Confirm Vertex was in RemainingVertices
#endif
				SortedRange.Add(Vertex);
			}
			Swap(NewIndependentVertices, IndependentVertices);
		}

		// Shuffle the input according to the SortOrder found by the graph search
		TArray<ElementType> CopyOriginal;
		CopyOriginal.Reserve(NumElements);
		for (ElementType& Element : UniqueRange)
		{
			CopyOriginal.Add(MoveTemp(Element));
		}
		int32 SourceIndex = 0;
		for (ElementType& TargetElement : UniqueRange)
		{
			TargetElement = MoveTemp(CopyOriginal[SortedRange[SourceIndex++]]);
		}

		return true;
	}
}

namespace AlgoImpl
{
	/**
	 * Convert UniqueRange and GetElementDependencies into handles,
	 * dependency count, dependencies, and referencers
	 */
	template <typename RangeType, typename GetElementDependenciesType>
	inline void KahnTopologicalSort_CreateWorkingGraph(FKahnContext& Context, RangeType& UniqueRange, 
		GetElementDependenciesType GetElementDependencies, TSet<FKahnHandle>& OutInitialIndependents)
	{
		using ElementType = typename TElementType<RangeType>::Type;

		TArray<TArray<FKahnHandle>>& Referencers(Context.Referencers);
		TArray<TArray<FKahnHandle>>& Dependencies(Context.Dependencies);
		TArray<int32>& DependencyCount(Context.DependencyCount);

		int32 NumElements = GetNum(UniqueRange);
		TMap<ElementType, FKahnHandle> HandleOfElement;
		HandleOfElement.Reserve(NumElements);
		FKahnHandle Handle = 0;
		for (const ElementType& Element : UniqueRange)
		{
			FKahnHandle& ExistingHandle = HandleOfElement.FindOrAdd(Element, INDEX_NONE);
			check(ExistingHandle == INDEX_NONE);
			ExistingHandle = Handle++;
		}

		Referencers.SetNum(NumElements);
		Dependencies.SetNum(NumElements);
		DependencyCount.SetNum(NumElements);
		Handle = 0;
		for (const ElementType& Element : UniqueRange)
		{
			const auto& DependenciesInput = ::Invoke(GetElementDependencies, Element);
			TArray<FKahnHandle>& UniqueElementDependencies = Dependencies[Handle];

			for (const ElementType& Dependency : DependenciesInput)
			{
				FKahnHandle* DependencyHandle = HandleOfElement.Find(Dependency);
				if (DependencyHandle)
				{
					UniqueElementDependencies.Add(*DependencyHandle);
				}
			}
			Algo::Sort(UniqueElementDependencies);
			UniqueElementDependencies.SetNum(Algo::Unique(UniqueElementDependencies), EAllowShrinking::No);
			int32 NumUniqueDependencies = UniqueElementDependencies.Num();
			DependencyCount[Handle] = NumUniqueDependencies;
			if (NumUniqueDependencies == 0)
			{
				OutInitialIndependents.Add(Handle);
			}
			for (FKahnHandle DependencyHandle : UniqueElementDependencies)
			{
				TArray<FKahnHandle>& ElementReferencers = Referencers[DependencyHandle];
				ElementReferencers.Add(Handle);
			}
			++Handle;
		}
	}

	/**
	 * Called when there is a MutuallyReachableVertexSet (aka no vertices are independent). It returns the most-independent
	 * maximal MutuallyReachableVertexSet.
	 */
	inline const TSet<FKahnHandle>& FindMostIndependentMutuallyReachableVertexSet(FKahnContext& Context)
	{
		if (!Context.MRVSContext.IsSet())
		{
			int32 NumVertices = Context.RemainingVertices.Num();
			FMutuallyReachableVertexSetContext& InitContext = Context.MRVSContext.Emplace();
			InitContext.VisitedReferences.Reserve(NumVertices);
			InitContext.VisitedDependencies.Reserve(NumVertices);
			InitContext.Culprits.Reserve(NumVertices);
			InitContext.Stack.Reserve(NumVertices);
			InitContext.Cycle.Reserve(NumVertices);
		}
		const TArray<TArray<FKahnHandle>>& Dependencies = Context.Dependencies;
		const TArray<TArray<FKahnHandle>>& Referencers = Context.Referencers;
		FMutuallyReachableVertexSetContext& MRVSContext = *Context.MRVSContext;
		TSet<FKahnHandle>& VisitedReferences = MRVSContext.VisitedReferences;
		TSet<FKahnHandle>& VisitedDependencies = MRVSContext.VisitedDependencies;
		TSet<FKahnHandle>& MaximalMRVS = MRVSContext.MaximalMRVS;
		TSet<FKahnHandle>& Culprits = MRVSContext.Culprits;
		TArray<FKahnHandle>& Stack = MRVSContext.Stack;
		TArray<FKahnHandle>& Cycle = MRVSContext.Cycle;

		// Copy Remaining Vertices into Culprits; we will be modifying the list
		// as we search for the most-independent MRVS.
		Culprits.Reset();
		Culprits.Append(Context.RemainingVertices);

		// Start from an arbitrary vertex
		check(!Culprits.IsEmpty());
		FKahnHandle StartingVertex = *Culprits.CreateIterator();

		int32 StartingVertexLoopCount = 0;
		while (StartingVertex != INDEX_NONE)
		{
			// Find a cycle by arbitrarily following dependencies until we revisit a vertex.
			VisitedReferences.Reset();
			Stack.Reset();
			VisitedReferences.Add(StartingVertex);
			Stack.Add(StartingVertex);
			FKahnHandle CycleWitnessVertex = StartingVertex;
			bool bCycleWitnessWasAlreadyInStack = false;
			while (!bCycleWitnessWasAlreadyInStack)
			{
				FKahnHandle NextVertex = INDEX_NONE;
				for (FKahnHandle Dependency : Dependencies[CycleWitnessVertex])
				{
					if (Culprits.Contains(Dependency)) // Only consider edges to remaining vertices
					{
						NextVertex = Dependency;
						break;
					}
				}
				// Assert a dependency is found. This function is only called when every vertex has remaining dependencies.
				check(NextVertex != INDEX_NONE);
				CycleWitnessVertex = NextVertex;
				VisitedReferences.Add(CycleWitnessVertex, &bCycleWitnessWasAlreadyInStack);
				Stack.Add(CycleWitnessVertex);
			}

			// The cycle is everything on the stack after the first occurrence of CycleWitnessVertex.
			// Cycle stacks will be like [7,7] or [7,3,2,7], always starting and ending with a vertex
			// Pop the second occurrence of CycleWitnessVertex off the stack
			check(Stack.Num() >= 2);
			Stack.Pop(EAllowShrinking::No);
			// Find the start of the Cycle
			int32 StartIndex = Stack.Num() - 1;
			while (StartIndex >= 0 && Stack[StartIndex] != CycleWitnessVertex)
			{
				--StartIndex;
			}
			check(StartIndex >= 0);
			Cycle.Reset();
			Cycle.Append(TArrayView<FKahnHandle>(Stack).Mid(StartIndex));

			// We now have a cycle, which is an MRVS that may or may not be maximal. Expand it to a maximal MRVS by
			// intersecting everything reachable from it with everything from which it is reachable.

			// Find all references to the cycle. We start from VisitedReferences and Stack that we created when
			// finding the cycle, since everything in that set is a referencer of the cycle
			while (!Stack.IsEmpty())
			{
				FKahnHandle Vertex = Stack.Pop(EAllowShrinking::No);
				for (FKahnHandle Referencer : Referencers[Vertex])
				{
					if (Culprits.Contains(Referencer)) // Only consider edges to remaining vertices
					{
						bool bAlreadyVisited;
						VisitedReferences.Add(Referencer, &bAlreadyVisited);
						if (!bAlreadyVisited)
						{
							Stack.Add(Referencer);
						}
					}
				}
			}

			// Find all dependencies from the cycle.
			VisitedDependencies.Reset();
			Stack.Reset();
			for (FKahnHandle Vertex : Cycle)
			{
				VisitedDependencies.Add(Vertex);
				Stack.Add(Vertex);
			}
			while (!Stack.IsEmpty())
			{
				FKahnHandle Vertex = Stack.Pop(EAllowShrinking::No);
				for (FKahnHandle Dependency : Dependencies[Vertex])
				{
					if (Culprits.Contains(Dependency)) // Only consider edges to remaining vertices
					{
						bool bAlreadyVisited;
						VisitedDependencies.Add(Dependency, &bAlreadyVisited);
						if (!bAlreadyVisited)
						{
							Stack.Add(Dependency);
						}
					}
				}
			}

			MaximalMRVS = VisitedDependencies.Intersect(VisitedReferences);

			// We now have a maximal MRVS, but there may be multiple maximal MRVS's, and we need to find the most independent
			// one. Remove all elements of VisitedReferences from the remaining vertices in Culprits. We will not need to consider
			// them when searching the graph for the more independent MRVS. Once removed, look for any dependencies from
			// the MaximalMRVS to a remaining culprit. If any exist, then we can follow them to find a new maximal MRVS that
			// is more independent than the current maximal MRVS.
			for (FKahnHandle Vertex : VisitedReferences)
			{
				Culprits.Remove(Vertex);
			}

			StartingVertex = INDEX_NONE;
			for (FKahnHandle Vertex : MaximalMRVS)
			{
				for (FKahnHandle Dependency : Dependencies[Vertex])
				{
					if (Culprits.Contains(Dependency)) // Only consider edges to remaining vertices
					{
						StartingVertex = Dependency;
						break;
					}
				}
				if (StartingVertex != INDEX_NONE)
				{
					break;
				}
			}

			// If we found a new StartingVertex, the loop will continue and follow it to the new MaximalMRVS
			// Otherwise the current MaximalMRVS is our returned value
			checkf(StartingVertexLoopCount++ < Context.RemainingVertices.Num(), TEXT("Infinite loop in FindMostIndependentMutuallyReachableVertexSet"));
		}

		return MaximalMRVS;
	}
}
