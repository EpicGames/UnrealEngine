// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/Less.h"
#include "Templates/UnrealTemplate.h" // For GetData, GetNum


namespace AlgoImpl
{
	/**
	 * Sort elements using user defined predicate class. The sort is unstable, meaning that the ordering of equal items is not necessarily preserved.
	 * This is the internal sorting function used by Sort overrides.
	 * This used to be Algo::Sort and is now considered legacy.
	 *
	 * @param	First	pointer to the first element to sort
	 * @param	Num		the number of items to sort
	 * @param Predicate predicate class
	 */
	template <typename T, typename ProjectionType, typename PredicateType> 
	void LegacySortInternal(T* First, int32 Num, ProjectionType Projection, PredicateType Predicate)
	{
		struct FStack
		{
			T* Min;
			T* Max;
		};

		if( Num < 2 )
		{
			return;
		}
		FStack RecursionStack[32]={{First,First+Num-1}}, Current, Inner;
		for( FStack* StackTop=RecursionStack; StackTop>=RecursionStack; --StackTop )
		{
			Current = *StackTop;
		Loop:
			PTRINT Count = Current.Max - Current.Min + 1;
			if( Count <= 8 )
			{
				// Use simple bubble-sort.
				while( Current.Max > Current.Min )
				{
					T *Max, *Item;
					for( Max=Current.Min, Item=Current.Min+1; Item<=Current.Max; Item++ )
					{
						if( Predicate( Invoke( Projection, *Max ), Invoke( Projection, *Item ) ) )
						{
							Max = Item;
						}
					}
					Swap( *Max, *Current.Max-- );
				}
			}
			else
			{
				// Grab middle element so sort doesn't exhibit worst-cast behavior with presorted lists.
				Swap( Current.Min[Count/2], Current.Min[0] );

				// Divide list into two halves, one with items <=Current.Min, the other with items >Current.Max.
				Inner.Min = Current.Min;
				Inner.Max = Current.Max+1;
				for( ; ; )
				{
					while( ++Inner.Min<=Current.Max && !Predicate( Invoke( Projection, *Current.Min ), Invoke( Projection, *Inner.Min ) ) );
					while( --Inner.Max> Current.Min && !Predicate( Invoke( Projection, *Inner.Max ), Invoke( Projection, *Current.Min ) ) );
					if( Inner.Min>Inner.Max )
					{
						break;
					}
					Swap( *Inner.Min, *Inner.Max );
				}
				Swap( *Current.Min, *Inner.Max );

				// Save big half and recurse with small half.
				if( Inner.Max-1-Current.Min >= Current.Max-Inner.Min )
				{
					if( Current.Min+1 < Inner.Max )
					{
						StackTop->Min = Current.Min;
						StackTop->Max = Inner.Max - 1;
						StackTop++;
					}
					if( Current.Max>Inner.Min )
					{
						Current.Min = Inner.Min;
						goto Loop;
					}
				}
				else
				{
					if( Current.Max>Inner.Min )
					{
						StackTop->Min = Inner  .Min;
						StackTop->Max = Current.Max;
						StackTop++;
					}
					if( Current.Min+1<Inner.Max )
					{
						Current.Max = Inner.Max - 1;
						goto Loop;
					}
				}
			}
		}
	}
}

namespace Algo
{
	/**
	 * Sort a range of elements using its operator<.  The sort is unstable.
	 * This used to be Algo::Sort and is now considered legacy.
	 *
	 * @param  Range  The range to sort.
	 */
	template <typename RangeType>
	UE_REWRITE void LegacySort(RangeType&& Range)
	{
		AlgoImpl::LegacySortInternal(GetData(Range), GetNum(Range), FIdentityFunctor(), TLess<>());
	}

	/**
	 * Sort a range of elements using a user-defined predicate class.  The sort is unstable.
	 * This used to be Algo::Sort and is now considered legacy.
	 *
	 * @param  Range      The range to sort.
	 * @param  Predicate  A binary predicate object used to specify if one element should precede another.
	 */
	template <typename RangeType, typename PredicateType>
	UE_REWRITE void LegacySort(RangeType&& Range, PredicateType Pred)
	{
		AlgoImpl::LegacySortInternal(GetData(Range), GetNum(Range), FIdentityFunctor(), MoveTemp(Pred));
	}

	/**
	 * Sort a range of elements by a projection using the projection's operator<.  The sort is unstable.
	 * This used to be Algo::Sort and is now considered legacy.
	 *
	 * @param  Range  The range to sort.
	 * @param  Proj   The projection to sort by when applied to the element.
	 */
	template <typename RangeType, typename ProjectionType>
	UE_REWRITE void LegacySortBy(RangeType&& Range, ProjectionType Proj)
	{
		AlgoImpl::LegacySortInternal(GetData(Range), GetNum(Range), MoveTemp(Proj), TLess<>());
	}

	/**
	 * Sort a range of elements by a projection using a user-defined predicate class.  The sort is unstable.
	 * This used to be Algo::Sort and is now considered legacy.
	 *
	 * @param  Range      The range to sort.
	 * @param  Proj       The projection to sort by when applied to the element.
	 * @param  Predicate  A binary predicate object, applied to the projection, used to specify if one element should precede another.
	 */
	template <typename RangeType, typename ProjectionType, typename PredicateType>
	UE_REWRITE void LegacySortBy(RangeType&& Range, ProjectionType Proj, PredicateType Pred)
	{
		AlgoImpl::LegacySortInternal(GetData(Range), GetNum(Range), MoveTemp(Proj), MoveTemp(Pred));
	}
}
