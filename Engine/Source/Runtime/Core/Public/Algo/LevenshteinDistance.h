// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Templates/Invoke.h"

namespace Algo
{
	/**
	* LevenshteinDistance return the number of edit operation we need to transform RangeA to RangeB.
	* Operation type are Add/Remove/substitution of range element. Base on Levenshtein algorithm.
	*
	* Range[A/B]Type: Support [] operator and the range element must be able to be compare with == operator
	*                 Support GetNum() functionality
	*
	* @param RangeA			The first range of element
	* @param RangeB			The second range of element
	* @return				The number of operation to transform RangeA to RangeB
	*/
	template <typename RangeAType, typename RangeBType>
	[[nodiscard]] int32 LevenshteinDistance(const RangeAType& RangeA, const RangeBType& RangeB)
	{
		const int32 LenA = GetNum(RangeA);
		const int32 LenB = GetNum(RangeB);
		//Early return for empty string
		if (LenA == 0)
		{
			return LenB;
		}
		else if (LenB == 0)
		{
			return LenA;
		}

		auto DataA = GetData(RangeA);
		auto DataB = GetData(RangeB);

		TArray<int32> OperationCount;
		//Initialize data
		OperationCount.AddUninitialized(LenB + 1);
		for (int32 IndexB = 0; IndexB <= LenB; ++IndexB)
		{
			OperationCount[IndexB] = IndexB;
		}
		//find the operation count
		for (int32 IndexA = 0; IndexA < LenA; ++IndexA)
		{
			int32 LastCount = IndexA + 1;
			for (int32 IndexB = 0; IndexB < LenB; ++IndexB)
			{
				int32 NewCount = OperationCount[IndexB];
				if (DataA[IndexA] != DataB[IndexB])
				{
					NewCount = FMath::Min3(NewCount, LastCount, OperationCount[IndexB+1]) + 1;
				}
				OperationCount[IndexB] = LastCount;
				LastCount = NewCount;
			}
			OperationCount[LenB] = LastCount;
		}
		return OperationCount[LenB];
	}

} //End namespace Algo
