// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Impl/RangePointerType.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/Less.h"
#include "Templates/UnrealTemplate.h" // For MoveTemp

namespace AlgoImpl
{
	template <typename RangeType, typename ProjectionType, typename PredicateType>
	constexpr typename TRangePointerType<RangeType>::Type MaxElementBy(RangeType& Range, ProjectionType Proj, PredicateType Pred)
	{
		typename TRangePointerType<RangeType>::Type Result = nullptr;

		for (auto& Elem : Range)
		{
			if (!Result || Invoke(Pred, Invoke(Proj, *Result), Invoke(Proj, Elem)))
			{
				Result = &Elem;
			}
		}

		return Result;
	}
}

namespace Algo
{
	/**
	 * Returns a pointer to the maximum element in a range.
	 * If the range contains multiple maximum elements, a pointer to the first one will be returned.
	 *
	 * @param  Range  The range to find the maximum element in.
	 * @return A pointer to the maximum element, or nullptr if the range was empty.
	 */
	template <typename RangeType>
	[[nodiscard]] UE_REWRITE constexpr auto MaxElement(RangeType&& Range)
		-> decltype(AlgoImpl::MaxElementBy(Range, FIdentityFunctor(), TLess<>()))
	{
		return AlgoImpl::MaxElementBy(Range, FIdentityFunctor(), TLess<>());
	}

	/**
	 * Returns a pointer to the maximum element in a range with a user-defined binary comparator.
	 * If the range contains multiple maximum elements, a pointer to the first one will be returned.
	 *
	 * @param  Range  The range to find the maximum element in.
	 * @param  Comp   The comparator to use when comparing two elements.
	 * @return A pointer to the maximum element, or nullptr if the range was empty.
	 */
	template <typename RangeType, typename ComparatorType>
	[[nodiscard]] UE_REWRITE auto MaxElement(RangeType&& Range, ComparatorType Comp)
		-> decltype(AlgoImpl::MaxElementBy(Range, FIdentityFunctor(), MoveTemp(Comp)))
	{
		return AlgoImpl::MaxElementBy(Range, FIdentityFunctor(), MoveTemp(Comp));
	}

	/**
	 * Returns a pointer to the maximum element in a range with a user-defined binary comparator.
	 * If the range contains multiple maximum elements, a pointer to the first one will be returned.
	 *
	 * @param  Range  The range to find the maximum element in.
	 * @param  Proj   The projection to apply to the element to use for comparison.
	 * @return A pointer to the maximum element, or nullptr if the range was empty.
	 */
	template <typename RangeType, typename ProjectionType>
	[[nodiscard]] UE_REWRITE auto MaxElementBy(RangeType&& Range, ProjectionType Proj)
		-> decltype(AlgoImpl::MaxElementBy(Range, MoveTemp(Proj), TLess<>()))
	{
		return AlgoImpl::MaxElementBy(Range, MoveTemp(Proj), TLess<>());
	}

	/**
	 * Returns a pointer to the maximum element in a range with a user-defined binary comparator.
	 * If the range contains multiple maximum elements, a pointer to the first one will be returned.
	 *
	 * @param  Range  The range to find the maximum element in.
	 * @param  Proj   The projection to apply to the element to use for comparison.
	 * @param  Comp   The comparator to use when comparing two elements.
	 * @return A pointer to the maximum element, or nullptr if the range was empty.
	 */
	template <typename RangeType, typename ProjectionType, typename ComparatorType>
	[[nodiscard]] UE_REWRITE auto MaxElementBy(RangeType&& Range, ProjectionType Proj, ComparatorType Comp)
		-> decltype(AlgoImpl::MaxElementBy(Range, MoveTemp(Proj), MoveTemp(Comp)))
	{
		return AlgoImpl::MaxElementBy(Range, MoveTemp(Proj), MoveTemp(Comp));
	}
}
