// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Impl/RangePointerType.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/Less.h"
#include "Templates/UnrealTemplate.h" // For MoveTemp

namespace AlgoImpl
{
	template <typename RangeType, typename ProjectionType, typename PredicateType>
	typename TRangePointerType<RangeType>::Type MinElementBy(RangeType& Range, ProjectionType Proj, PredicateType Pred)
	{
		typename TRangePointerType<RangeType>::Type Result = nullptr;

		for (auto& Elem : Range)
		{
			if (!Result || Invoke(Pred, Invoke(Proj, Elem), Invoke(Proj, *Result)))
			{
				Result = &Elem;
			}
		}

		return Result;
	}
}

namespace Algo
{
	/**
	 * Returns a pointer to the minimum element in a range.
	 * If the range contains multiple minimum elements, a pointer to the first one will be returned.
	 *
	 * @param  Range  The range to find the minimum element in.
	 * @return A pointer to the minimum element, or nullptr if the range was empty.
	 */
	template <typename RangeType>
	[[nodiscard]] UE_REWRITE auto MinElement(RangeType&& Range)
		-> decltype(AlgoImpl::MinElementBy(Range, FIdentityFunctor(), TLess<>()))
	{
		return AlgoImpl::MinElementBy(Range, FIdentityFunctor(), TLess<>());
	}

	/**
	 * Returns a pointer to the minimum element in a range with a user-defined binary comparator.
	 * If the range contains multiple minimum elements, a pointer to the first one will be returned.
	 *
	 * @param  Range  The range to find the minimum element in.
	 * @param  Comp   The comparator to use when comparing two elements.
	 * @return A pointer to the minimum element, or nullptr if the range was empty.
	 */
	template <typename RangeType, typename ComparatorType>
	[[nodiscard]] UE_REWRITE auto MinElement(RangeType&& Range, ComparatorType Comp)
		-> decltype(AlgoImpl::MinElementBy(Range, FIdentityFunctor(), MoveTemp(Comp)))
	{
		return AlgoImpl::MinElementBy(Range, FIdentityFunctor(), MoveTemp(Comp));
	}

	/**
	 * Returns a pointer to the minimum element in a range with a user-defined binary comparator.
	 * If the range contains multiple minimum elements, a pointer to the first one will be returned.
	 *
	 * @param  Range  The range to find the minimum element in.
	 * @param  Proj   The projection to apply to the element to use for comparison.
	 * @return A pointer to the minimum element, or nullptr if the range was empty.
	 */
	template <typename RangeType, typename ProjectionType>
	[[nodiscard]] UE_REWRITE auto MinElementBy(RangeType&& Range, ProjectionType Proj)
		-> decltype(AlgoImpl::MinElementBy(Range, MoveTemp(Proj), TLess<>()))
	{
		return AlgoImpl::MinElementBy(Range, MoveTemp(Proj), TLess<>());
	}

	/**
	 * Returns a pointer to the minimum element in a range with a user-defined binary comparator.
	 * If the range contains multiple minimum elements, a pointer to the first one will be returned.
	 *
	 * @param  Range  The range to find the minimum element in.
	 * @param  Proj   The projection to apply to the element to use for comparison.
	 * @param  Comp   The comparator to use when comparing two elements.
	 * @return A pointer to the minimum element, or nullptr if the range was empty.
	 */
	template <typename RangeType, typename ProjectionType, typename ComparatorType>
	[[nodiscard]] UE_REWRITE auto MinElementBy(RangeType&& Range, ProjectionType Proj, ComparatorType Comp)
		-> decltype(AlgoImpl::MinElementBy(Range, MoveTemp(Proj), MoveTemp(Comp)))
	{
		return AlgoImpl::MinElementBy(Range, MoveTemp(Proj), MoveTemp(Comp));
	}
}
