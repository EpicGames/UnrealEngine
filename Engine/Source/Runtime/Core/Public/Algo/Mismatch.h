// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/EqualTo.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/UnrealTemplate.h"

#include <type_traits>

namespace Algo::Private
{
	template <typename InputAType, typename InputBType, typename ProjectionType, typename PredicateType>
	constexpr auto Mismatch(const InputAType& InputA, const InputBType& InputB, ProjectionType Projection, PredicateType Predicate)
	{
		using IndexType = std::common_type_t<decltype(GetNum(InputA)), decltype(GetNum(InputB))>;

		IndexType Index = 0;

		const IndexType SizeA = (IndexType)GetNum(InputA);
		const IndexType SizeB = (IndexType)GetNum(InputB);

		auto* A = GetData(InputA);
		auto* B = GetData(InputB);

		for (;;)
		{
			if (Index == SizeA || Index == SizeB)
			{
				break;
			}

			if (!Invoke(Predicate, Invoke(Projection, *A++), Invoke(Projection, *B++)))
			{
				break;
			}

			++Index;
		}

		return Index;
	}
}

namespace Algo
{
	/**
	 * Returns the index at which two contiguous containers differ, using operator== to compare pairs of elements.
	 *
	 * @param  InputA     Container of elements that are used as the first argument to operator==.
	 * @param  InputB     Container of elements that are used as the second argument to operator==.
	 *
	 * @return The index at which the containers differ.
	 */
	template <typename InputAType, typename InputBType>
	[[nodiscard]] constexpr auto Mismatch(const InputAType& InputA, const InputBType& InputB) -> std::common_type_t<decltype(GetNum(InputA)), decltype(GetNum(InputB))>
	{
		return Private::Mismatch(InputA, InputB, FIdentityFunctor(), TEqualTo<>());
	}

	/**
	 * Returns the index at which two contiguous containers differ, using operator== to compare pairs of elements.
	 *
	 * @param  InputA     Container of elements that are used as the first argument to operator==.
	 * @param  InputB     Container of elements that are used as the second argument to operator==.
	 * @param  Predicate  Condition which returns true for elements which are deemed equal.
	 *
	 * @return The index at which the containers differ.
	 */
	template <typename InputAType, typename InputBType, typename PredicateType>
	[[nodiscard]] constexpr auto Mismatch(const InputAType& InputA, const InputBType& InputB, PredicateType Predicate) -> std::common_type_t<decltype(GetNum(InputA)), decltype(GetNum(InputB))>
	{
		return Private::Mismatch(InputA, InputB, FIdentityFunctor(), MoveTemp(Predicate));
	}

	/**
	 * Returns the index at which two contiguous containers differ, using operator== to compare pairs of projected elements.
	 *
	 * @param  InputA     Container of elements that are used as the first argument to operator==.
	 * @param  InputB     Container of elements that are used as the second argument to operator==.
	 * @param  Projection Projection to apply to the elements before comparing them.
	 *
	 * @return The index at which the containers differ.
	 */
	template <typename InputAType, typename InputBType, typename ProjectionType>
	[[nodiscard]] constexpr auto MismatchBy(const InputAType& InputA, const InputBType& InputB, ProjectionType Projection) -> std::common_type_t<decltype(GetNum(InputA)), decltype(GetNum(InputB))>
	{
		return Private::Mismatch(InputA, InputB, MoveTemp(Projection), TEqualTo<>());
	}

	/**
	 * Returns the index at which two contiguous containers differ, using a predicate to compare pairs of projected elements.
	 *
	 * @param  InputA     Container of elements that are used as the first argument to the predicate.
	 * @param  InputB     Container of elements that are used as the second argument to the predicate.
	 * @param  Projection Projection to apply to the elements before comparing them.
	 * @param  Predicate  Condition which returns true for elements which are deemed equal.
	 *
	 * @return The index at which the containers differ.
	 */
	template <typename InputAType, typename InputBType, typename ProjectionType, typename PredicateType>
	[[nodiscard]] constexpr auto MismatchBy(const InputAType& InputA, const InputBType& InputB, ProjectionType Projection, PredicateType Predicate) -> std::common_type_t<decltype(GetNum(InputA)), decltype(GetNum(InputB))>
	{
		return Private::Mismatch(InputA, InputB, MoveTemp(Projection), MoveTemp(Predicate));
	}
}
