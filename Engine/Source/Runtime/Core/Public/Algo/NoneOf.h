// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Algo/Common.h"
#include "Templates/Invoke.h"


namespace Algo
{
	/**
	 * Checks if no element in the range is truthy.
	 *
	 * @param  Range  The range to check.
	 *
	 * @return  true if no element is truthy, false otherwise.
	 */
	template <typename RangeType>
	[[nodiscard]] bool NoneOf(const RangeType& Range)
	{
		for (const auto& Element : Range)
		{
			if (Element)
			{
				return false;
			}
		}

		return true;
	}

	/**
	 * Checks if no projection of the elements in the range is truthy.
	 *
	 * @param  Range       The range to check.
	 * @param  Projection  The projection to apply to each element.
	 *
	 * @return  true if none of the projections are truthy, false otherwise.
	 */
	template <typename RangeType, typename ProjectionType>
	[[nodiscard]] bool NoneOf(const RangeType& Range, ProjectionType Projection)
	{
		for (const auto& Element : Range)
		{
			if (Invoke(Projection, Element))
			{
				return false;
			}
		}

		return true;
	}

	/**
	 * Checks if no projection of the elements in the range is truthy.
	 * Should be used when the range iterator doesn't return a reference.
	 *
	 * @param  Range       The range to check.
	 * @param  Projection  The projection to apply to each element.
	 *
	 * @return  true if none of the projections are truthy, false otherwise.
	 */
	template <typename RangeType, typename ProjectionType>
	[[nodiscard]] bool NoneOf(const RangeType& Range, ProjectionType Projection, ENoRef NoRef)
	{
		for (const auto Element : Range)
		{
			if (Invoke(Projection, Element))
			{
				return false;
			}
		}

		return true;
	}
}
