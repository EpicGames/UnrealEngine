// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace Algo
{
	/**
	 * Rearranges the elements so that all the elements for which Predicate returns true precede all those for which it returns false.  (not stable)
	 *
	 * @param	First		pointer to the first element
	 * @param	Num			the number of items
	 * @param	Predicate	unary predicate class
	 * @return	index of the first element in the second group
	 */
	template<class T, typename IndexType, typename UnaryPredicate>
	IndexType Partition(T* Elements, const IndexType Num, UnaryPredicate Predicate)
	{
		T* First = Elements;
		T* Last = Elements + Num;
		
		while (First != Last) 
		{
			while (Predicate(*First)) 
			{
				++First;
				if (First == Last) 
				{	
					return (IndexType)(First - Elements);
				}
			}
		
			do 
			{
				--Last;
				if (First == Last)
				{
					return (IndexType)(First - Elements);
				}
			} while (!Predicate(*Last));
		
			Exchange(*First, *Last);
			++First;
		}
	
		return (IndexType)(First - Elements);
	}

	/**
	 * Rearranges the elements so that all the elements for which Predicate returns true precede all those for which it returns false.  (not stable)
	 *
	 * @param	Range		the range to sort
	 * @param	Predicate	a unary predicate object
	 * @return	index of the first element in the second group
	 */
	template <typename RangeType, typename UnaryPredicateType>
	UE_REWRITE auto Partition(RangeType&& Range, UnaryPredicateType Predicate) -> decltype(GetNum(Range))
	{
		return Partition(GetData(Range), GetNum(Range), MoveTemp(Predicate));
	}
} //namespace Algo