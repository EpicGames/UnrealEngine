// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/RandomStream.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/UnrealTemplate.h"

namespace AlgoImpl
{
	template <typename RangeType, typename RandomFunc>
	void RandomShuffle(RangeType&& Range, RandomFunc Random)
	{
		auto Data = GetData(Range);

		using SizeType = decltype(GetNum(Range));
		const SizeType Num = GetNum(Range);

		for (SizeType Index = 0; Index < Num - 1; ++Index)
		{
			// Get a random integer in [Index, Num)
			const SizeType RandomIndex = Index + (SizeType)::Invoke(Random, Num - Index);
			if (RandomIndex != Index)
			{
				Swap(Data[Index], Data[RandomIndex]);
			}
		}
	}
}

namespace Algo
{
	/**
	 * Randomly shuffle a range of elements.
	 *
	 * @param  Range  Any contiguous container.
	 */
	template <typename RangeType>
	void RandomShuffle(RangeType&& Range)
	{
		using SizeType = decltype(GetNum(Range));
		AlgoImpl::RandomShuffle(Range, [] (const SizeType MaxIndex) { return FMath::RandHelper64(MaxIndex); });
	}

	/**
	 * Randomly shuffle a range of elements.
	 *
	 * @param  Range  Any contiguous container.
	 * @param  RandomStream - Random stream from which to retrieve random indices.
	 */
	template <typename RangeType>
	void RandomShuffle(RangeType&& Range, FRandomStream& RandomStream)
	{
		using SizeType = decltype(GetNum(Range));
		AlgoImpl::RandomShuffle(Range, [&RandomStream] (const SizeType MaxIndex) { return RandomStream.RandHelper64(MaxIndex); });
	}
}
