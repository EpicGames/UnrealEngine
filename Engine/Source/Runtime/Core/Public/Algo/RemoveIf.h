// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Invoke.h"

namespace Algo
{
	/**
	 * Moves all elements which do not match the predicate to the front of the range, while leaving all
	 * other elements is a constructed but unspecified state.  The elements which were not removed are
	 * not guaranteed to be kept in order (unstable).
	 *
	 * @param  Range  The range of elements to manipulate.
	 * @param  Pred   A callable which maps elements to truthy values, specifying elements to be removed.
	 *
	 * @return The number of elements that should be kept.
	 */
	template <typename RangeType, typename Predicate>
	[[nodiscard]] int32 RemoveIf(RangeType& Range, Predicate Pred)
	{
		auto* First = GetData(Range);
		auto* Last  = First + GetNum(Range);

		auto* IterStart = First;
		auto* IterEnd   = Last;
		for (;;)
		{
			// Skip non-removed elements at the start
			for (;;)
			{
				if (IterStart == IterEnd)
				{
					return UE_PTRDIFF_TO_INT32(IterStart - First);
				}

				if (Invoke(Pred, *IterStart))
				{
					break;
				}

				++IterStart;
			}

			// Skip removed elements at the end
			for (;;)
			{
				--IterEnd;

				if (IterStart == IterEnd)
				{
					return UE_PTRDIFF_TO_INT32(IterStart - First);
				}

				if (!Invoke(Pred, *IterEnd))
				{
					break;
				}
			}

			*IterStart = MoveTemp(*IterEnd);

			++IterStart;
		}
	}

	/**
	 * Moves all elements which do not match the predicate to the front of the range, while leaving all
	 * other elements is a constructed but unspecified state.  The elements which were not removed are
	 * guaranteed to be kept in order (stable).
	 *
	 * @param  Range  The range of elements to manipulate.
	 * @param  Pred   A callable which maps elements to truthy values, specifying elements to be removed.
	 *
	 * @return The number of elements that should be kept.
	 */
	template <typename RangeType, typename Predicate>
	[[nodiscard]] int32 StableRemoveIf(RangeType&& Range, Predicate Pred)
	{
		auto* First = GetData(Range);
		auto* Last  = First + GetNum(Range);

		auto* IterStart = First;

		// Skip non-removed elements at the start
		for (;;)
		{
			if (IterStart == Last)
			{
				return UE_PTRDIFF_TO_INT32(IterStart - First);
			}

			if (Invoke(Pred, *IterStart))
			{
				break;
			}

			++IterStart;
		}

		auto* IterKeep = IterStart;
		++IterKeep;

		for (;;)
		{
			if (IterKeep == Last)
			{
				return UE_PTRDIFF_TO_INT32(IterStart - First);
			}

			if (!Invoke(Pred, *IterKeep))
			{
				*IterStart++ = MoveTemp(*IterKeep);
			}

			++IterKeep;
		}
	}
}
