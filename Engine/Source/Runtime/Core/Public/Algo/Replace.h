// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Invoke.h"
#include "Templates/UnrealTemplate.h"

namespace Algo
{
	/**
	 * Replaces all elements that compare equal to one value with a new value.
	 *
	 * @param Range The range to search and modify in-place.
	 * @param InOld The value to search for.
	 * @param InNew The value to copy in as a replacement.
	 */
	template <typename RangeType, typename ValueType>
	void Replace(RangeType&& Range, const ValueType& InOld, const ValueType& InNew)
	{
		for (auto& Elem : Forward<RangeType>(Range))
		{
			if (Elem == InOld)
			{
				Elem = InNew;
			}
		}
	}

	/**
	 * Replaces all elements that satisfy the predicate with the given value.
	 *
	 * @param Range The range to search and modify in-place.
	 * @param InPred The predicate to apply to each element.
	 * @param InNew The value to copy in as a replacement for each element satisfying the predicate.
	 */
	template <typename RangeType, typename ValueType, typename PredicateType>
	void ReplaceIf(RangeType&& Range, PredicateType InPred, const ValueType& InNew)
	{
		for (auto& Elem : Forward<RangeType>(Range))
		{
			if (Invoke(InPred, Elem))
			{
				Elem = InNew;
			}
		}
	}
}
