// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UnrealTemplate.h" // for Swap

namespace AlgoImpl
{
	template <typename T>
	void Reverse(T* Array, int32 ArraySize)
	{
		for (int32 i = 0, i2 = ArraySize - 1; i < ArraySize / 2 /*rounding down*/; ++i, --i2)
		{
			Swap(Array[i], Array[i2]);
		}
	}
}

namespace Algo
{
	/**
	 * Reverses a range
	 *
	 * @param  Array  The array to reverse.
	 */
	template <typename T, int32 ArraySize>
	UE_REWRITE void Reverse(T (&Array)[ArraySize])
	{
		return AlgoImpl::Reverse((T*)Array, ArraySize);
	}

	/**
	 * Reverses a range
	 *
	 * @param  Array      A pointer to the array to reverse
	 * @param  ArraySize  The number of elements in the array.
	 */
	template <typename T>
	UE_REWRITE void Reverse(T* Array, int32 ArraySize)
	{
		return AlgoImpl::Reverse(Array, ArraySize);
	}

	/**
	 * Reverses a range
	 *
	 * @param  Container  The container to reverse
	 */
	template <typename ContainerType>
	UE_REWRITE void Reverse(ContainerType&& Container)
	{
		return AlgoImpl::Reverse(Container.GetData(), Container.Num());
	}
}
