// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UnrealTemplate.h" // For GetData, GetNum, Swap


namespace AlgoImpl
{
	template <typename T>
	int32 RotateInternal(T* First, int32 Num, int32 Count)
	{
		if (Count == 0)
		{
			return Num;
		}

		if (Count >= Num)
		{
			return 0;
		}

		T* Iter = First;
		T* Mid  = First + Count;
		T* End  = First + Num;

		T* OldMid = Mid;
		for (;;)
		{
			Swap(*Iter++, *Mid++);
			if (Mid == End)
			{
				if (Iter == OldMid)
				{
					return Num - Count;
				}

				Mid = OldMid;
			}
			else if (Iter == OldMid)
			{
				OldMid = Mid;
			}
		}
	}
}

namespace Algo
{
	/**
	 * Rotates a given amount of elements from the front of the range to the end of the range.
	 *
	 * @param  Range  The range to rotate.
	 * @param  Num    The number of elements to rotate from the front of the range.
	 *
	 * @return The new index of the element that was previously at the start of the range.
	 */
	template <typename RangeType>
	UE_REWRITE int32 Rotate(RangeType&& Range, int32 Count)
	{
		return AlgoImpl::RotateInternal(GetData(Range), GetNum(Range), Count);
	}
}
