// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Accumulate.h"
#include "Algo/Impl/RangePointerType.h"
#include "Math/RandomStream.h"
#include "Templates/UnrealTemplate.h"
#include "Traits/ElementType.h"
#include "GenericPlatform/GenericPlatformMath.h"

namespace AlgoImpl
{
	template <typename RangeType, typename ProjectionType>
	[[nodiscard]] typename TRangePointerType<typename TRemoveReference<RangeType>::Type>::Type SelectRandomWeightedBy(RangeType&& Range, ProjectionType Proj)
	{
		using WeightType = std::decay_t<TInvokeResult_T<ProjectionType, TElementType_T<RangeType>>>;

		const auto SumOfAllDesires = Algo::Accumulate(Range, (WeightType)0, [&Proj](auto Acc, auto&& Elem)
		{
			const auto Weight = Invoke(Proj, Elem);
			// Negative values are invalid and should be ignored
			return Acc + (Weight < (WeightType)0 ? (WeightType)0 : Weight);
		});
		
		auto RandomWeightedAvg = (WeightType)(FMath::FRand() * SumOfAllDesires);

		for (auto&& Elem : Forward<RangeType>(Range))
		{
			const auto Weight = Invoke(Proj, Elem);

			// Negative- or zero-weighted elements are never chosen, and are not subtracted from the total since they are not added above.
			if (Weight <= (WeightType)0)
			{
				continue;
			}

			if (RandomWeightedAvg < Weight)
			{
				return &Elem;
			}

			RandomWeightedAvg -= Weight;
		}

		return nullptr;
	}
}

namespace Algo
{
	/**
	 * Randomly select an element from a range of elements, weighted by a projection.
	 * The chance of any element being chosen is its weight / the sum of all the weights in the range.
	 * Negative- or zero- weighted elements will not be chosen or count toward the total.
	 *
	 * @param  Range  The range to select from. Can be any iterable type.
	 * @param  Proj   The projection to weight the random selection by. Should yield a numeric type.
	 */
	template <typename RangeType, typename ProjectionType>
	UE_REWRITE auto SelectRandomWeightedBy(RangeType&& Range, ProjectionType Proj)
		-> decltype(AlgoImpl::SelectRandomWeightedBy(Forward<RangeType>(Range), MoveTemp(Proj)))
	{
		return AlgoImpl::SelectRandomWeightedBy(Forward<RangeType>(Range), MoveTemp(Proj));
	}
}
