// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/IntroSort.h"


namespace Algo
{
	/**
	 * Sort a range of elements using its operator<.  The sort is unstable.
	 *
	 * @param  Range  The range to sort.
	 */
	template <typename RangeType>
	UE_REWRITE void Sort(RangeType&& Range)
	{
		IntroSort(Forward<RangeType>(Range));
	}

	/**
	 * Sort a range of elements using a user-defined predicate class.  The sort is unstable.
	 *
	 * @param  Range      The range to sort.
	 * @param  Predicate  A binary predicate object used to specify if one element should precede another.
	 */
	template <typename RangeType, typename PredicateType>
	UE_REWRITE void Sort(RangeType&& Range, PredicateType Pred)
	{
		IntroSort(Forward<RangeType>(Range), MoveTemp(Pred));
	}

	/**
	 * Sort a range of elements by a projection using the projection's operator<.  The sort is unstable.
	 *
	 * @param  Range  The range to sort.
	 * @param  Proj   The projection to sort by when applied to the element.
	 */
	template <typename RangeType, typename ProjectionType>
	UE_REWRITE void SortBy(RangeType&& Range, ProjectionType Proj)
	{
		IntroSortBy(Forward<RangeType>(Range), MoveTemp(Proj));
	}

	/**
	 * Sort a range of elements by a projection using a user-defined predicate class.  The sort is unstable.
	 *
	 * @param  Range      The range to sort.
	 * @param  Proj       The projection to sort by when applied to the element.
	 * @param  Predicate  A binary predicate object, applied to the projection, used to specify if one element should precede another.
	 */
	template <typename RangeType, typename ProjectionType, typename PredicateType>
	UE_REWRITE void SortBy(RangeType&& Range, ProjectionType Proj, PredicateType Pred)
	{
		IntroSortBy(Forward<RangeType>(Range), MoveTemp(Proj), MoveTemp(Pred));
	}
}
