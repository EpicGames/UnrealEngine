// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/BinarySearch.h"
#include "Algo/Rotate.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/Less.h"
#include "Templates/UnrealTemplate.h" // For GetData, GetNum, Swap


namespace AlgoImpl
{
	template <typename T, typename ProjectionType, typename PredicateType>
	void Merge(T* First, int32 Mid, int32 Num, ProjectionType Projection, PredicateType Predicate)
	{
		int32 AStart = 0;
		int32 BStart = Mid;

		while (AStart < BStart && BStart < Num)
		{
			int32 NewAOffset = AlgoImpl::UpperBoundInternal(First + AStart, BStart - AStart, Invoke(Projection, First[BStart]), Projection, Predicate);
			AStart += NewAOffset;

			if (AStart >= BStart)
			{
				return;
			}

			int32 NewBOffset = AlgoImpl::LowerBoundInternal(First + BStart, Num - BStart, Invoke(Projection, First[AStart]), Projection, Predicate);
			AlgoImpl::RotateInternal(First + AStart, NewBOffset + BStart - AStart, BStart - AStart);
			BStart += NewBOffset;
			AStart += NewBOffset + 1;
		}
	}

	inline constexpr int32 MinMergeSubgroupSize = 2;

	/**
	 * Sort elements using user defined projection and predicate classes.  The sort is stable, meaning that the ordering of equal items is preserved.
	 * This is the internal sorting function used by the Algo::Sort overloads.
	 *
	 * @param  First       Pointer to the first element to sort.
	 * @param  Num         The number of items to sort.
	 * @param  Projection  A projection to apply to each element to get the value to sort by.
	 * @param  Predicate   A predicate class which compares two projected elements and returns whether one occurs before the other.
	 */
	template <typename T, typename ProjectionType, typename PredicateType>
	void StableSortInternal(T* First, int32 Num, ProjectionType Projection, PredicateType Predicate)
	{
		int32 SubgroupStart = 0;

		if constexpr (MinMergeSubgroupSize > 1)
		{
			if constexpr (MinMergeSubgroupSize > 2)
			{
				// First pass with simple bubble-sort.
				do
				{
					int32 GroupEnd = SubgroupStart + MinMergeSubgroupSize;
					if (Num < GroupEnd)
					{
						GroupEnd = Num;
					}
					do
					{
						for (int32 It = SubgroupStart; It < GroupEnd - 1; ++It)
						{
							if (Invoke(Predicate, Invoke(Projection, First[It + 1]), Invoke(Projection, First[It])))
							{
								Swap(First[It], First[It + 1]);
							}
						}
						GroupEnd--;
					}
					while (GroupEnd - SubgroupStart > 1);

					SubgroupStart += MinMergeSubgroupSize;
				}
				while (SubgroupStart < Num);
			}
			else
			{
				for (int32 Subgroup = 0; Subgroup < Num; Subgroup += 2)
				{
					if (Subgroup + 1 < Num && Invoke(Predicate, Invoke(Projection, First[Subgroup + 1]), Invoke(Projection, First[Subgroup])))
					{
						Swap(First[Subgroup], First[Subgroup + 1]);
					}
				}
			}
		}

		int32 SubgroupSize = MinMergeSubgroupSize;
		while (SubgroupSize < Num)
		{
			SubgroupStart = 0;
			do
			{
				int32 MergeNum = SubgroupSize << 1;
				if (Num - SubgroupStart < MergeNum)
				{
					MergeNum = Num - SubgroupStart;
				}

				Merge(First + SubgroupStart, SubgroupSize, MergeNum, Projection, Predicate);
				SubgroupStart += SubgroupSize << 1;
			}
			while (SubgroupStart < Num);

			SubgroupSize <<= 1;
		}
	}
}

namespace Algo
{
	/**
	 * Sort a range of elements using its operator<.  The sort is stable.
	 *
	 * @param  Range  The range to sort.
	 */
	template <typename RangeType>
	UE_REWRITE void StableSort(RangeType&& Range)
	{
		AlgoImpl::StableSortInternal(GetData(Range), GetNum(Range), FIdentityFunctor(), TLess<>());
	}

	/**
	 * Sort a range of elements using a user-defined predicate class.  The sort is stable.
	 *
	 * @param  Range      The range to sort.
	 * @param  Predicate  A binary predicate object used to specify if one element should precede another.
	 */
	template <typename RangeType, typename PredicateType>
	UE_REWRITE void StableSort(RangeType&& Range, PredicateType Pred)
	{
		AlgoImpl::StableSortInternal(GetData(Range), GetNum(Range), FIdentityFunctor(), MoveTemp(Pred));
	}

	/**
	 * Sort a range of elements by a projection using the projection's operator<.  The sort is stable.
	 *
	 * @param  Range  The range to sort.
	 * @param  Proj   The projection to sort by when applied to the element.
	 */
	template <typename RangeType, typename ProjectionType>
	UE_REWRITE void StableSortBy(RangeType&& Range, ProjectionType Proj)
	{
		AlgoImpl::StableSortInternal(GetData(Range), GetNum(Range), MoveTemp(Proj), TLess<>());
	}

	/**
	 * Sort a range of elements by a projection using a user-defined predicate class.  The sort is stable.
	 *
	 * @param  Range      The range to sort.
	 * @param  Proj       The projection to sort by when applied to the element.
	 * @param  Predicate  A binary predicate object, applied to the projection, used to specify if one element should precede another.
	 */
	template <typename RangeType, typename ProjectionType, typename PredicateType>
	UE_REWRITE void StableSortBy(RangeType&& Range, ProjectionType Proj, PredicateType Pred)
	{
		AlgoImpl::StableSortInternal(GetData(Range), GetNum(Range), MoveTemp(Proj), MoveTemp(Pred));
	}
}
