// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Tuple.h"

namespace Algo
{
	namespace Private
	{
		template <typename... OutputTypes>
		struct TTiedTupleAdder
		{
			template <typename TupleType>
			void Add(TupleType&& Values)
			{
				static_assert(TIsTuple<TupleType>::Value);
				static_assert(sizeof...(OutputTypes) == TTupleArity<typename TDecay<TupleType>::Type>::Value, "Wrong length of tuple for output");

				VisitTupleElements(
					[](auto& Output, auto&& Value)
					{
						Output.Add(Forward<decltype(Value)>(Value));
					},
					Outputs,
					Forward<TupleType>(Values)
				);
			}

			TTuple<OutputTypes&...> Outputs;
		};
	}
	

	/**
	 * Ties n objects with an Add function, usually containers, into one object with an Add function accepting an n-tuple
	 * that forwards the Add calls.
	 *
	 * This is useful for algorithms such as Algo::Transform and Algo::Copy.
	 *
	 * Example:
	 *	TArray<int32> Out1;
	 *	TArray<FString> Out2;
	 *	TieTupleAdd(Out1, Out2).Add(MakeTuple(42, TEXT("42")));
	 *	// Out1 = { 42 }, Out2 = { "42" }
	 *
	 *	@param Outputs Objects with an Add() function
	 *	@return An object with an Add function that will call Add on the tied elements
	 */
	template <typename... OutputTypes>
	[[nodiscard]] Private::TTiedTupleAdder<OutputTypes...> TieTupleAdd(OutputTypes&... Outputs)
	{
		return { { Outputs... } };
	}
}