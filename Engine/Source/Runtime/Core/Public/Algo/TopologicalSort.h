// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/KahnTopologicalSort.h"
#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"

namespace Algo
{
	/**
	 * Sorts the given range in leaf to root order:
	 * For every pair of elements (A,B) where SortedIndex(A) < SortedIndex(B),
	 * either B is not reachable from A, or A and B are mutually reachable.
	 *
	 * @param UniqueRange A range with element type T
	 *        Type T must support GetTypeHash and copy+move constructors; T being pointertype is recommended.
	 *        In/Out Variable, is sorted in place. Will be unmodified if function returns false.
	 * @param GetElementDependencies A callable with prototype that is one of
	 *            RangeType<T> GetElementDependencies(const T& Element)
	 *            const RangeType<T>& GetElementDependencies(const T& Element)
	 *        It should return the leaf-ward vertices of directed edges from the root-wards Element.
	 * @param Flags
	 *        ETopologicalSort::AllowCycles: If present, cycles will be allowed, with the caveat that the sorted
	 *            ordering of MutuallyReachable vertices in the sorted range is arbitrary.
	 *            If not present, the presence of a cycle will cause a failure to sort.
	 * 
	 * @return True if succeeded, false if failed due to cycles.
	 */
	template <typename RangeType, typename GetElementDependenciesType>
	UE_REWRITE bool TopologicalSort(RangeType&& UniqueRange, GetElementDependenciesType GetElementDependencies,
		ETopologicalSort Flags = ETopologicalSort::None)
	{
		return KahnTopologicalSort(Forward<RangeType>(UniqueRange), Forward<GetElementDependenciesType>(GetElementDependencies), Flags);
	}
}