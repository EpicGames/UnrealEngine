// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Invoke.h"

/*
 * Inspired by std::transform.
 *
 * Simple example:
 *	TArray<FString> Out;
 *	TArray<int32> Inputs { 1, 2, 3, 4, 5, 6 };
 *	Algo::Transform(
 *		Inputs,
 *		Out,
 *		[](int32 Input) { return LexToString(Input); }
 *	);
 *	// Out1 == [ "1", "2", "3", "4", "5", "6" ]
 *
 * You can also use Transform to output to multiple targets that have an Add function, by using :
 * 
 *	TArray<FString> Out1;
 *	TArray<int32> Out2;
 *
 *	TArray<int32> Inputs = { 1, 2, 3, 4, 5, 6 };
 *	Algo::Transform(
 *		Inputs,
 *		TiedTupleAdd(Out1, Out2),
 *		[](int32 Input) { return ForwardAsTuple(LexToString(Input), Input * Input); }
 *	);
 *
 *	// Out1 == [ "1", "2", "3", "4", "5", "6" ]
 *	// Out2 == [ 1, 4, 9, 16, 25, 36 ]
 */
namespace Algo
{
	/**
	 * Conditionally applies a transform to a range and stores the results into a container
	 *
	 * @param  Input      Any iterable type
	 * @param  Output     Container to hold the output
	 * @param  Predicate  Condition which returns true for elements that should be transformed and false for elements that should be skipped
	 * @param  Trans      Transformation operation
	 */
	template <typename InT, typename OutT, typename PredicateT, typename TransformT>
	void TransformIf(const InT& Input, OutT&& Output, PredicateT Predicate, TransformT Trans)
	{
		for (const auto& Value : Input)
		{
			if (Invoke(Predicate, Value))
			{
				Output.Add(Invoke(Trans, Value));
			}
		}
	}

	/**
	 * Applies a transform to a range and stores the results into a container
	 *
	 * @param  Input   Any iterable type
	 * @param  Output  Container to hold the output
	 * @param  Trans   Transformation operation
	 */
	template <typename InT, typename OutT, typename TransformT>
	void Transform(const InT& Input, OutT&& Output, TransformT Trans)
	{
		for (const auto& Value : Input)
		{
			Output.Add(Invoke(Trans, Value));
		}
	}
}
