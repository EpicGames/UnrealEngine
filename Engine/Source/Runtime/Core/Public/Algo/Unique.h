// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UnrealTemplate.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/EqualTo.h"

namespace AlgoImpl
{
	template <typename T, typename SizeType, typename ProjectionType, typename BinaryPredicate>
	SizeType Unique(T* Array, SizeType ArraySize, ProjectionType Proj, BinaryPredicate Predicate)
	{
		if (ArraySize <= 1)
		{
			return ArraySize;
		}

		T* Result = Array;
		for (T* Iter = Array + 1; Iter != Array + ArraySize; ++Iter)
		{
			if (!Invoke(Predicate, Invoke(Proj, *Result), Invoke(Proj, *Iter))) 
			{
				++Result;
				if (Result != Iter)
				{
					*Result = MoveTemp(*Iter);
				}
			}
		}

		return static_cast<SizeType>(Result + 1 - Array);
	}
}

namespace Algo
{
	/**
	 * Eliminates all but the first element from every consecutive group of equivalent elements and
	 * returns past-the-end index of unique elements for the new logical end of the range.
	 *
	 * Removing is done by shifting the elements in the range in such a way that elements to be erased are overwritten.
	 * Relative order of the elements that remain is preserved and the physical size of the range is unchanged.
	 * References to an element between the new logical end and the physical end of the range are still
	 * dereferenceable, but the elements themselves have unspecified values. A call to `Unique` is typically followed by a call
	 * to a container's `SetNum` method as:
	 *
	 * ```Container.SetNum(Algo::Unique(Container));```
	 *
	 * that erases the unspecified values and reduces the physical size of the container
	 * to match its new logical size.
	 *
	 * Elements are compared using operator== or given binary predicate. The behavior is undefined if it is not an equivalence relation.
	 *
	 * See https://en.cppreference.com/w/cpp/algorithm/unique
	 */
	template<typename RangeType>
	[[nodiscard]] auto Unique(RangeType&& Range) -> decltype(AlgoImpl::Unique(GetData(Range), GetNum(Range), FIdentityFunctor(), TEqualTo<>{}))
	{
		return AlgoImpl::Unique(GetData(Range), GetNum(Range), FIdentityFunctor(), TEqualTo<>{});
	}

	template<typename RangeType, typename BinaryPredicate>
	[[nodiscard]] auto Unique(RangeType&& Range, BinaryPredicate Predicate) -> decltype(AlgoImpl::Unique(GetData(Range), GetNum(Range), FIdentityFunctor(), MoveTemp(Predicate)))
	{
		return AlgoImpl::Unique(GetData(Range), GetNum(Range), FIdentityFunctor(), MoveTemp(Predicate));
	}

	template<typename RangeType, typename ProjectionType>
	[[nodiscard]] auto UniqueBy(RangeType&& Range, ProjectionType Proj) -> decltype(AlgoImpl::Unique(GetData(Range), GetNum(Range), MoveTemp(Proj), TEqualTo<>{}))
	{
		return AlgoImpl::Unique(GetData(Range), GetNum(Range), MoveTemp(Proj), TEqualTo<>{});
	}

	template<typename RangeType, typename ProjectionType, typename BinaryPredicate>
	[[nodiscard]] auto UniqueBy(RangeType&& Range, ProjectionType Proj, BinaryPredicate Predicate) -> decltype(AlgoImpl::Unique(GetData(Range), GetNum(Range), MoveTemp(Proj), MoveTemp(Predicate)))
	{
		return AlgoImpl::Unique(GetData(Range), GetNum(Range), MoveTemp(Proj), MoveTemp(Predicate));
	}
}
