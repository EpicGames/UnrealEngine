// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if defined( PVS_STUDIO )

	//-V::505,542,630,677,704,719,720,730,735,780,1002,1055,1062,1100
	//-V:check(:501,547,560,605
	//-V:checkSlow(:547
	//-V:checkf(:510
	//-V:checkfSlow(:547
	//-V:dtAssert(:568
	//-V:rcAssert(:568
	//-V:ENABLE_LOC_TESTING:560,617
	//-V:WITH_EDITOR:560
	//-V:UE_LOG_ACTIVE:560
	//-V:verify:501
	//-V:UE_BUILD_SHIPPING:501
	//-V:WITH_EDITOR:501
	//-V:TestTrueExpr:501
	//-V:DECLARE_LOG_CATEGORY_EXTERN:501
	//-V:PLATFORM_:517,547
	//-V:ensureMsgf:562
	//-V:MAX_VERTS_PER_POLY:512
	//-V:<<:614
	//-V:SPAWN_INIT:595
	//-V:BEGIN_UPDATE_LOOP:595
	//-V:OPENGL_PERFORMANCE_DATA_INVALID:560,564
	//-V:UE_CLOG(:501,560
	//-V:UE_CLOGF(:501,560
	//-V:UE_LOG(:501,510,560
	//-V:UE_LOGF(:501,510,560
	//-V:ensure(:595
	//-V:ALLOCATE_VERTEX_DATA_TEMPLATE:501
	//-V:UGL_REQUIRED:501
	//-V:DEBUG_LOG_HTTP:523
	//-V:GIsEditor:560
	//-V:GEventDrivenLoaderEnabled:501
	//-V:IMPLEMENT_AI_INSTANT_TEST:773
	//-V:ENABLE_VERIFY_GL:564
	//-V:INC_MEMORY_STAT_BY:568
	//-V:DEC_MEMORY_STAT_BY:568
	//-V:SELECT_STATIC_MESH_VERTEX_TYPE:622
	//-V:This(:678
	//-V:state->error:649
	//-V:PERF_DETAILED_PER_CLASS_GC_STATS:686
	//-V:FMath:656
	//-V:->*:607
	//-V:CalcSegmentCostOnPoly:764
	//-V:DrawLine:764
	//-V:VertexData:773
	//-V:Linker:678
	//-V:self:678
	//-V:FindChar:679
	//-V:BytesToHex:530
	//-V:FindOrAdd:530
	//-V:GetPawn<:623
	//-V:Response.Response:1051

	// The following classes retain a reference to data supplied in the constructor by the derived class which can not yet be initialized.
	//-V:FMemoryWriter(:1050
	//-V:FObjectWriter(:1050
	//-V:FDurationTimer(:1050
	//-V:FScopedDurationTimer(:1050
	//-V:FQueryFastData(:1050

	// Exclude all generated protobuf files
	//V_EXCLUDE_PATH *.pb.cc

	// Disabling because incorrectly flagging all TStaticArrays
	// V557: Array overrun is possible
	//-V::557

	// Disabling because too many virtuals currently in use in constructors/destructors, need to revist
	// V1053: Calling the 'foo' virtual function in the constructor/destructor may lead to unexpected result at runtime.
	//-V::1053

	// 7.42 false positive
	//-V:NAME_None:502

#endif // #if defined( PVS_STUDIO )
