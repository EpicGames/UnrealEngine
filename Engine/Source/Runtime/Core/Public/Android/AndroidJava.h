// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if USE_ANDROID_JNI
#include "Android/AndroidJavaEnv.h"
#include <jni.h>

/*
Wrappers for Java classes.
TODO: make function look up static? It doesn't matter for our case here (we only create one instance of the class ever)
*/
struct FJavaClassMethod
{
	FName		Name;
	FName		Signature;
	jmethodID	Method;
};

class FJavaClassObject
{
public:
	static FJavaClassObject GetGameActivity();

	// !!  All Java objects returned by JNI functions are local references.
	FJavaClassObject(FName ClassName, const char* CtorSig, ...);
	FJavaClassObject(jclass LocalClass, jobject LocalObject);
	FJavaClassObject(const FJavaClassObject&) = delete;
	~FJavaClassObject();

	FJavaClassObject& operator=(const FJavaClassObject&) = delete;
	
	FJavaClassMethod GetClassMethod(const char* MethodName, const char* FuncSig);

	// TODO: Define this for extra cases
	template<typename ReturnType>
	ReturnType CallMethod(FJavaClassMethod Method, ...);

	UE_FORCEINLINE_HINT jobject GetJObject() const
	{
		return Object;
	}

	static FScopedJavaObject<jstring> GetJString(const FString& String);

	static void VerifyException();

protected:

	jobject			Object;
	jclass			Class;
};

template<>
void FJavaClassObject::CallMethod<void>(FJavaClassMethod Method, ...);

template<>
bool FJavaClassObject::CallMethod<bool>(FJavaClassMethod Method, ...);

template<>
int FJavaClassObject::CallMethod<int>(FJavaClassMethod Method, ...);

template<>
jobject FJavaClassObject::CallMethod<jobject>(FJavaClassMethod Method, ...);

template<>
jobjectArray FJavaClassObject::CallMethod<jobjectArray>(FJavaClassMethod Method, ...);

template<>
int64 FJavaClassObject::CallMethod<int64>(FJavaClassMethod Method, ...);

template<>
FString FJavaClassObject::CallMethod<FString>(FJavaClassMethod Method, ...);

template<>
float FJavaClassObject::CallMethod<float>(FJavaClassMethod Method, ...);

namespace UE::Jni
{
	template <typename T>
	struct TClassObject: FJavaClassObject
	{
		template <typename... TArgs>
		TClassObject(const TArgs&... args)
			: FJavaClassObject{Jni::Class<T>, *[this, &args...]
			{
				auto LocalObject = T::New(Jni::Class<T>, args...);
				VerifyException();
				return LocalObject;
			}()}
		{
		}
	};
}
#endif
