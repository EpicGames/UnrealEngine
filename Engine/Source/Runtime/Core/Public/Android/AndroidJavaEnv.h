// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_ANDROID_JNI

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "Containers/StringConv.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include <algorithm>
#include <type_traits>
#include <jni.h>

namespace AndroidJavaEnv
{
	// Returns the java environment
	UE_DEPRECATED(5.7, "Use InitializeJavaEnv() instead.")
	CORE_API void InitializeJavaEnv(JavaVM* VM, jint Version, jobject GlobalThis);
	CORE_API void InitializeJavaEnv();
	CORE_API jobject GetGameActivityThis();
	CORE_API jobject GetClassLoader();
	//UE_DEPRECATED(5.7, "Use UE::Jni::Env global instead.")
	CORE_API JNIEnv* GetJavaEnv(bool bRequireGlobalThis = true);
	CORE_API jclass FindJavaClass(const char* name);
	CORE_API jclass FindJavaClassGlobalRef(const char* name);
	UE_DEPRECATED(5.7, "Don't call this.")
	CORE_API void DetachJavaEnv();
	CORE_API bool CheckJavaException();
}

#define UE_JNI_MEMBER(Member) \
	::UE::Jni::FMember{Member, #Member}

#define UE_JNI_NATIVE_METHOD(Method) \
	::UE::Jni::FNativeMethod{#Method, ::UE::Jni::Signature<decltype(Method)>.GetData(), (void*)Method}

template <typename T> requires std::is_pointer_v<T>
class FScopedJavaObject;

namespace UE::Jni
{
    template <typename T>
	struct TTraits
	{
	};

	template <typename T>
	struct TSignature
	{
		static constexpr FAnsiStringView Value = TTraits<T>::Signature;
	};

	template <typename T>
	inline constexpr FAnsiStringView Signature = TSignature<T>::Value;
	
	template <typename TRet, typename... TArgs>
	struct TSignature<TRet (TArgs...)>
	{
	private:
		static constexpr auto _Value = []
		{
			TStaticArray<char, (3 + ... + Signature<TArgs>.Len()) + Signature<TRet>.Len()> Value;
			auto Iterator = Value.begin();

			*Iterator++ = '(';
			((Iterator = std::ranges::copy(Signature<TArgs>, Iterator).out), ...);
			*Iterator++ = ')';
			Iterator = std::ranges::copy(Signature<TRet>, Iterator).out;
			*Iterator = '\0';

			return Value;
		}();

	public:
		static constexpr FAnsiStringView Value{_Value.GetData(), _Value.Num() - 1};
	};

	template <typename TRet, typename T, typename... TArgs>
	struct TSignature<TRet (JNIEnv*, T, TArgs...)>: TSignature<TRet (TArgs...)>
	{
	};

	template <typename T, typename... TArgs>
	struct TConstructor;
	
	template <typename T, typename U>
	struct TMember;
	
	struct FMember
	{
		const char* Name;
		const char* Signature;
		enum EType
		{
			StaticField,
			Field,
			StaticMethod,
			Method
		}
		Type;
		union
		{
			jfieldID* FieldId;
			jmethodID* MethodId;
		};
		
		template <typename T, typename... TArgs>
		UE_DEPRECATED(5.8, "TConstructor<T(TArgs...) instead.")
		constexpr FMember(TConstructor<T, TArgs...>& Constructor, const char* = nullptr)
			: Name{"<init>"}
			, Signature{Jni::Signature<void (TArgs...)>.GetData()}
			, Type{Method}
			, MethodId{&Constructor.Value}
		{
		}
		
		template <typename T, typename... TArgs>
		constexpr FMember(TConstructor<T(TArgs...)>& Constructor, const char* = nullptr)
			: Name{"<init>"}
			, Signature{Jni::Signature<void (TArgs...)>.GetData()}
			, Type{Method}
			, MethodId{&Constructor.Value}
		{
		}
		
		template <typename T, typename U> requires std::is_function_v<U>
		constexpr FMember(TMember<T, U>& Member, const char* _Name)
			: Name{_Name}
			, Signature{Jni::Signature<U>.GetData()}
			, Type{std::is_pointer_v<T> ? Method : StaticMethod}
			, MethodId{&Member.Value}
		{
		}
		
		template <typename T, typename U>
		constexpr FMember(TMember<T, U>& Member, const char* _Name)
			: Name{_Name}
			, Signature{Jni::Signature<U>.GetData()}
			, Type{std::is_pointer_v<T> ? Field : StaticField}
			, FieldId{&Member.Value}
		{
		}
	};
	
	using FNativeMethod = JNINativeMethod;
	
	namespace Java::Lang
	{
		using FClass = std::remove_pointer_t<jclass>;
		using FObject = std::remove_pointer_t<jobject>;
		using FString = std::remove_pointer_t<jstring>;
		
		template <typename T, std::size_t Rank = 1> requires(Rank >= 1)
		struct TArray: std::remove_pointer_t<jobjectArray>
		{
		};
		
		template <typename T>
		struct TArray<T, 1>: std::remove_pointer_t<typename TTraits<T>::ArrayType>
		{
		};

		template <typename T>
		struct TClass: FClass
		{
		};
	}

	template <typename T>
	inline constexpr FAnsiStringView ClassName = T::ClassName;

	template <>
	inline constexpr FAnsiStringView ClassName<Java::Lang::FClass> = "java/lang/Class";

	template <>
	inline constexpr FAnsiStringView ClassName<Java::Lang::FObject> = "java/lang/Object";

	template <>
	inline constexpr FAnsiStringView ClassName<Java::Lang::FString> = "java/lang/String";
	
	template <typename T>
	struct TTraits<T*>
	{
	private:
		static constexpr auto _Signature = []
		{
			TStaticArray<char, 3 + ClassName<T>.Len()> Value;
			auto Iterator = Value.begin();

			*Iterator++ = 'L';
			Iterator = std::ranges::copy(ClassName<T>, Iterator).out;
			*Iterator++ = ';';
			*Iterator = '\0';

			return Value;
		}();

	public:
		static constexpr FAnsiStringView Signature{_Signature.GetData(), _Signature.Num() - 1};
		using ArrayType = jobjectArray;
	};

	template <>
	struct TTraits<void>
	{
		static constexpr FAnsiStringView Signature = "V";
	};

	template <>
	struct TTraits<jboolean>
	{
		static constexpr FAnsiStringView Signature = "Z";
		using ArrayType = jbooleanArray;
	};

	template <>
	struct TTraits<jbyte>
	{
		static constexpr FAnsiStringView Signature = "B";
		using ArrayType = jbyteArray;
	};

	template <>
	struct TTraits<jchar>
	{
		static constexpr FAnsiStringView Signature = "C";
		using ArrayType = jcharArray;
	};

	template <>
	struct TTraits<jshort>
	{
		static constexpr FAnsiStringView Signature = "S";
		using ArrayType = jshortArray;
	};

	template <>
	struct TTraits<jint>
	{
		static constexpr FAnsiStringView Signature = "I";
		using ArrayType = jintArray;
	};

	template <>
	struct TTraits<jlong>
	{
		static constexpr FAnsiStringView Signature = "J";
		using ArrayType = jlongArray;
	};

	template <>
	struct TTraits<jfloat>
	{
		static constexpr FAnsiStringView Signature = "F";
		using ArrayType = jfloatArray;
	};

	template <>
	struct TTraits<jdouble>
	{
		static constexpr FAnsiStringView Signature = "D";
		using ArrayType = jdoubleArray;
	};

	template <typename T, std::size_t Rank>
	struct TTraits<Java::Lang::TArray<T, Rank>*>
	{
	private:
		static constexpr auto _Signature = []
		{
			TStaticArray<char, 1 + Rank + Signature<T>.Len()> Value;
			auto Iterator = Value.begin();

			Iterator = std::fill_n(Iterator, Rank, '[');
			Iterator = std::ranges::copy(Signature<T>, Iterator).out;
			*Iterator = '\0';

			return Value;
		}();

	public:
		static constexpr FAnsiStringView Signature{_Signature.GetData(), _Signature.Num() - 1};
	};
	
	template <typename T>
	inline Java::Lang::TClass<T>* Class;

	inline thread_local const struct FEnv
	{
		FEnv();
		FEnv(const FEnv&) = delete;
		~FEnv();

		FEnv& operator=(const FEnv&) = delete;

		template <typename T, bool bIsOptional, typename U>
		void Initialize(Java::Lang::TClass<U>* Class) const
		{
			check(Class);
		
			if constexpr (requires { T::Members; })
			{
				bool bResult = Register(Class, T::Members);
	
				if constexpr (bIsOptional)
				{
					if (!bResult)
					{
						return;
					}
				}
				else
				{
					checkf(bResult, TEXT("Failed to register members for java class: %s"), StringCast<TCHAR>(U::ClassName.GetData()).Get());
				}
			}

			if constexpr (requires { T::NativeMethods; })
			{
				bool bResult = Register(Class, T::NativeMethods);
	
				if constexpr (bIsOptional)
				{
					if (!bResult)
					{
						return;
					}
				}
				else
				{
					checkf(bResult, TEXT("Failed to register native methods for java class: %s"), StringCast<TCHAR>(U::ClassName.GetData()).Get());
				}
			}
		}

		template <typename T, bool bIsOptional = requires { requires T::bIsOptional; }>
		void Initialize() const
		{
			if constexpr (std::is_base_of_v<Java::Lang::FObject, T>)
			{
				if (!Class<T>)
				{
					FScopedJavaObject<Java::Lang::TClass<T>*> LocalClass = Find<T>();

					if constexpr (bIsOptional)
					{
						if (!LocalClass)
						{
							return;
						}
					}
					else
					{
						checkf(LocalClass, TEXT("Failed to find java class: %s"), StringCast<TCHAR>(T::ClassName.GetData()).Get());
					}
				
					Class<T> = NewGlobalRef(*LocalClass);
				}

				Initialize<T, false>(Class<T>);
			}
			else
			{
				if (!Class<typename T::PartialClass>)
				{
					FScopedJavaObject<Java::Lang::TClass<typename T::PartialClass>*> LocalClass = Find<typename T::PartialClass>();

					if (!LocalClass)
					{
						return;
					}

					Class<typename T::PartialClass> = NewGlobalRef(*LocalClass);
				}
			
				Initialize<T, bIsOptional>(Class<typename T::PartialClass>);
			}
		}
		
		template <typename T>
		[[nodiscard]] T* NewGlobalRef(T* Object) const
		{
			static_assert(std::is_base_of_v<Java::Lang::FObject, T>); // FIXME needs ndk r28+, should be std::is_pointer_interconvertible_base_of_v
			
			return reinterpret_cast<T*>(Value->NewGlobalRef(Object));
		}
		
		template <typename T>
		[[nodiscard]] FScopedJavaObject<Java::Lang::TClass<T>*> Find() const
		{
			return FScopedJavaObject<Java::Lang::TClass<T>*>{FindClass(ClassName<T>.GetData())};
		}
		
		template <typename U, typename T>
		[[nodiscard]] TMember<T, U> GetStatic(Java::Lang::TClass<T>* Class, const char* Name) const
		{
			if constexpr (std::is_function_v<U>)
			{
				jmethodID MethodId = Value->GetStaticMethodID(Class, Name, Signature<U>.GetData());

				if (!MethodId)
				{
					Value->ExceptionClear();
				}

				return {MethodId};
			}
			else
			{
				jfieldID FieldId = Value->GetStaticFieldID(Class, Name, Signature<U>.GetData());

				if (!FieldId)
				{
					Value->ExceptionClear();
				}

				return {FieldId};
			}
		}
		
		template <typename U, typename T>
		[[nodiscard]] TMember<T*, U> Get(Java::Lang::TClass<T>* Class, const char* Name) const
		{
			if constexpr (std::is_function_v<U>)
			{
				jmethodID MethodId = Value->GetMethodID(Class, Name, Signature<U>.GetData());

				if (!MethodId)
				{
					Value->ExceptionClear();
				}

				return {MethodId};
			}
			else
			{
				jfieldID FieldId = Value->GetFieldID(Class, Name, Signature<U>.GetData());

				if (!FieldId)
				{
					Value->ExceptionClear();
				}

				return {FieldId};
			}
		}
		
		[[nodiscard]] FScopedJavaObject<jclass> FindClass(const char* ClassName) const;
		bool Register(jclass Class, TArrayView<const FMember> Members) const;
		bool Register(jclass Class, TArrayView<const FNativeMethod> NativeMethods) const;

		template <CContiguousRange T> requires(!std::is_convertible_v<const T&, FStringView>)
		auto Marshal(const T& Params) const
		{
			using U = decltype(Marshal(Params[0]))::Type;
			
			int32 Count = GetNum(Params);			
			FScopedJavaObject<Java::Lang::TArray<U>*> OutArray{reinterpret_cast<Java::Lang::TArray<U>*>(Value->NewObjectArray(Count, Class<std::remove_pointer_t<U>>, nullptr))};
			
			for (int32 Index = 0; Index < Count; ++Index)
			{
				Value->SetObjectArrayElement(*OutArray, Index, *Marshal(Params[Index]));
			}
			
			return OutArray;
		}

		FScopedJavaObject<Java::Lang::FString*> Marshal(FStringView Param) const;
		FString Marshal(Java::Lang::FString* Param) const;

		[[nodiscard]] JNIEnv* Get() const
		{
			return Value;
		}
		
		JNIEnv* operator->() const
		{
			return Value;
		}

		operator JNIEnv*() const
		{
			return Value;
		}
		
	private:
		friend void AndroidJavaEnv::InitializeJavaEnv();

		JNIEnv* Value;
		bool bAttached;
		
		void Initialize() const;
	}
	Env;

	template <typename T>
	struct TInitialize
	{
	private:
		[[gnu::retain, gnu::used, gnu::section("UE_Jni_Initialize")]]
		static constexpr void (FEnv::*Function)() const = &FEnv::Initialize<T>;
	};

	template <typename T, typename... TArgs> requires(!std::is_function_v<T>)
	struct UE_DEPRECATED(5.8, "TConstructor<T(TArgs...) instead.") TConstructor<T, TArgs...>: TConstructor<T(TArgs...)>
	{
	};

	template <typename T, typename... TArgs>
	struct TConstructor<T(TArgs...)>
	{
		jmethodID Value = nullptr;
		
		[[nodiscard]] FScopedJavaObject<T*> operator()(Java::Lang::TClass<T>* Class, TArgs... args) const
		{
			check(Value);
			return FScopedJavaObject<T*>{Env->NewObject(Class, Value, args...)};
		}
	};
	
	template <typename T, typename U>
	struct TMember<T, U*>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] FScopedJavaObject<U*> Get(Java::Lang::TClass<T>* Class) const
		{
			check(Value);
			jobject Result = Env->GetStaticObjectField(Class, Value);
			return !Env->ExceptionCheck() ? FScopedJavaObject<U*>{Result} : nullptr;
		}
		void Set(Java::Lang::TClass<T>* Class, U* arg) const
		{
			check(Value);
			Env->SetStaticObjectField(Class, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T, jboolean>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jboolean Get(Java::Lang::TClass<T>* Class) const
		{
			check(Value);
			return Env->GetStaticBooleanField(Class, Value);
		}
		void Set(Java::Lang::TClass<T>* Class, jboolean arg) const
		{
			check(Value);
			Env->SetStaticBooleanField(Class, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T, jbyte>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jbyte Get(Java::Lang::TClass<T>* Class) const
		{
			check(Value);
			return Env->GetStaticByteField(Class, Value);
		}
		void Set(Java::Lang::TClass<T>* Class, jbyte arg) const
		{
			check(Value);
			Env->SetStaticByteField(Class, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T, jchar>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jchar Get(Java::Lang::TClass<T>* Class) const
		{
			check(Value);
			return Env->GetStaticCharField(Class, Value);
		}
		void Set(Java::Lang::TClass<T>* Class, jchar arg) const
		{
			check(Value);
			Env->SetStaticCharField(Class, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T, jshort>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jshort Get(Java::Lang::TClass<T>* Class) const
		{
			check(Value);
			return Env->GetStaticShortField(Class, Value);
		}
		void Set(Java::Lang::TClass<T>* Class, jshort arg) const
		{
			check(Value);
			Env->SetStaticShortField(Class, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T, jint>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jint Get(Java::Lang::TClass<T>* Class) const
		{
			check(Value);
			return Env->GetStaticIntField(Class, Value);
		}
		void Set(Java::Lang::TClass<T>* Class, jint arg) const
		{
			check(Value);
			Env->SetStaticIntField(Class, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T, jlong>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jlong Get(Java::Lang::TClass<T>* Class) const
		{
			check(Value);
			return Env->GetStaticLongField(Class, Value);
		}
		void Set(Java::Lang::TClass<T>* Class, jlong arg) const
		{
			check(Value);
			Env->SetStaticLongField(Class, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T, jfloat>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jfloat Get(Java::Lang::TClass<T>* Class) const
		{
			check(Value);
			return Env->GetStaticFloatField(Class, Value);
		}
		void Set(Java::Lang::TClass<T>* Class, jfloat arg) const
		{
			check(Value);
			Env->SetStaticFloatField(Class, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T, jdouble>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jdouble Get(Java::Lang::TClass<T>* Class) const
		{
			check(Value);
			return Env->GetStaticDoubleField(Class, Value);
		}
		void Set(Java::Lang::TClass<T>* Class, jdouble arg) const
		{
			check(Value);
			Env->SetStaticDoubleField(Class, Value, arg);
		}
	};
	
	template <typename T, typename U>
	struct TMember<T*, U*>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] FScopedJavaObject<U*> Get(T* Object) const
		{
			check(Value);
			jobject Result = Env->GetObjectField(Object, Value);
			return !Env->ExceptionCheck() ? FScopedJavaObject<U*>{Result} : nullptr;
		}
		void Set(T* Object, U* arg) const
		{
			check(Value);
			Env->SetObjectField(Object, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T*, jboolean>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jboolean Get(T* Object) const
		{
			check(Value);
			return Env->GetBooleanField(Object, Value);
		}
		void Set(T* Object, jboolean arg) const
		{
			check(Value);
			Env->SetBooleanField(Object, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T*, jbyte>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jbyte Get(T* Object) const
		{
			check(Value);
			return Env->GetByteField(Object, Value);
		}
		void Set(T* Object, jbyte arg) const
		{
			check(Value);
			Env->SetByteField(Object, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T*, jchar>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jchar Get(T* Object) const
		{
			check(Value);
			return Env->GetCharField(Object, Value);
		}
		void Set(T* Object, jchar arg) const
		{
			check(Value);
			Env->SetCharField(Object, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T*, jshort>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jshort Get(T* Object) const
		{
			check(Value);
			return Env->GetShortField(Object, Value);
		}
		void Set(T* Object, jshort arg) const
		{
			check(Value);
			Env->SetShortField(Object, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T*, jint>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jint Get(T* Object) const
		{
			check(Value);
			return Env->GetIntField(Object, Value);
		}
		void Set(T* Object, jint arg) const
		{
			check(Value);
			Env->SetIntField(Object, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T*, jlong>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jlong Get(T* Object) const
		{
			check(Value);
			return Env->GetLongField(Object, Value);
		}
		void Set(T* Object, jlong arg) const
		{
			check(Value);
			Env->SetLongField(Object, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T*, jfloat>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jfloat Get(T* Object) const
		{
			check(Value);
			return Env->GetFloatField(Object, Value);
		}
		void Set(T* Object, jfloat arg) const
		{
			check(Value);
			Env->SetFloatField(Object, Value, arg);
		}
	};
	
	template <typename T>
	struct TMember<T*, jdouble>
	{
		jfieldID Value = nullptr;
		
		[[nodiscard]] jdouble Get(T* Object) const
		{
			check(Value);
			return Env->GetDoubleField(Object, Value);
		}
		void Set(T* Object, jdouble arg) const
		{
			check(Value);
			Env->SetDoubleField(Object, Value, arg);
		}
	};
	
	template <typename T, typename TRet, typename... TArgs>
	struct TMember<T, TRet* (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		FScopedJavaObject<TRet*> operator()(Java::Lang::TClass<T>* Class, TArgs... args) const
		{
			check(Value);
			jobject Result = Env->CallStaticObjectMethod(Class, Value, args...);
			return !Env->ExceptionCheck() ? FScopedJavaObject<TRet*>{Result} : nullptr;
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T, void (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		void operator()(Java::Lang::TClass<T>* Class, TArgs... args) const
		{
			check(Value);
			Env->CallStaticVoidMethod(Class, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T, jboolean (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jboolean operator()(Java::Lang::TClass<T>* Class, TArgs... args) const
		{
			check(Value);
			return Env->CallStaticBooleanMethod(Class, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T, jbyte (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jbyte operator()(Java::Lang::TClass<T>* Class, TArgs... args) const
		{
			check(Value);
			return Env->CallStaticByteMethod(Class, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T, jchar (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jchar operator()(Java::Lang::TClass<T>* Class, TArgs... args) const
		{
			check(Value);
			return Env->CallStaticCharMethod(Class, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T, jshort (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jshort operator()(Java::Lang::TClass<T>* Class, TArgs... args) const
		{
			check(Value);
			return Env->CallStaticShortMethod(Class, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T, jint (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jint operator()(Java::Lang::TClass<T>* Class, TArgs... args) const
		{
			check(Value);
			return Env->CallStaticIntMethod(Class, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T, jlong (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jlong operator()(Java::Lang::TClass<T>* Class, TArgs... args) const
		{
			check(Value);
			return Env->CallStaticLongMethod(Class, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T, jfloat (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jfloat operator()(Java::Lang::TClass<T>* Class, TArgs... args) const
		{
			check(Value);
			return Env->CallStaticFloatMethod(Class, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T, jdouble (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jdouble operator()(Java::Lang::TClass<T>* Class, TArgs... args) const
		{
			check(Value);
			return Env->CallStaticDoubleMethod(Class, Value, args...);
		}
	};
	
	template <typename T, typename TRet, typename... TArgs>
	struct TMember<T*, TRet* (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		FScopedJavaObject<TRet*> operator()(T* Object, TArgs... args) const
		{
			check(Value);
			jobject Result = Env->CallObjectMethod(Object, Value, args...);
			return !Env->ExceptionCheck() ? FScopedJavaObject<TRet*>{Result} : nullptr;
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T*, void (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		void operator()(T* Object, TArgs... args) const
		{
			check(Value);
			Env->CallVoidMethod(Object, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T*, jboolean (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jboolean operator()(T* Object, TArgs... args) const
		{
			check(Value);
			return Env->CallBooleanMethod(Object, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T*, jbyte (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jbyte operator()(T* Object, TArgs... args) const
		{
			check(Value);
			return Env->CallByteMethod(Object, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T*, jchar (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jchar operator()(T* Object, TArgs... args) const
		{
			check(Value);
			return Env->CallCharMethod(Object, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T*, jshort (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jshort operator()(T* Object, TArgs... args) const
		{
			check(Value);
			return Env->CallShortMethod(Object, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T*, jint (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jint operator()(T* Object, TArgs... args) const
		{
			check(Value);
			return Env->CallIntMethod(Object, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T*, jlong (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jlong operator()(T* Object, TArgs... args) const
		{
			check(Value);
			return Env->CallLongMethod(Object, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T*, jfloat (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jfloat operator()(T* Object, TArgs... args) const
		{
			check(Value);
			return Env->CallFloatMethod(Object, Value, args...);
		}
	};
	
	template <typename T, typename... TArgs>
	struct TMember<T*, jdouble (TArgs...)>
	{
		jmethodID Value = nullptr;
		
		jdouble operator()(T* Object, TArgs... args) const
		{
			check(Value);
			return Env->CallDoubleMethod(Object, Value, args...);
		}
	};

	namespace Java
	{
		namespace Lang
		{
			struct FClassLoader: FObject
			{
				static constexpr FAnsiStringView ClassName = "java/lang/ClassLoader";

				inline static TMember<FClassLoader*, jclass (jstring name)> loadClass;

				static constexpr FMember Members[]
				{
					UE_JNI_MEMBER(loadClass)
				};
			};
			
			struct FInteger: FObject
			{
				static constexpr FAnsiStringView ClassName = "java/lang/Integer";
		
				inline static TConstructor<FInteger(jint value)> New;
		
				static constexpr FMember Members[]
				{
					UE_JNI_MEMBER(New)
				};
			};
		}
		
		namespace Util
		{
			struct FHashMap: Lang::FObject
			{
				static constexpr FAnsiStringView ClassName = "java/util/HashMap";
		
				inline static TConstructor<FHashMap()> New;
				inline static TMember<FHashMap*, Lang::FObject* (Lang::FObject* key, Lang::FObject* value)> put;
		
				static constexpr FMember Members[]
				{
					UE_JNI_MEMBER(New),
					UE_JNI_MEMBER(put)
				};
			};
		}
	}
}

// Helper class that automatically calls DeleteLocalRef on the passed-in Java object when goes out of scope
template <typename T> requires std::is_pointer_v<T>
class FScopedJavaObject
{
public:
	using Type = T;
	
	FScopedJavaObject() = default;

	constexpr FScopedJavaObject(std::nullptr_t)
	{
	}
	
	explicit constexpr FScopedJavaObject(T Object)
		: ObjRef{Object}
	{
	}

	template <typename U> requires std::is_base_of_v<U, std::remove_pointer_t<T>> // FIXME needs ndk r28+, should be std::is_pointer_interconvertible_base_of_v
	explicit constexpr FScopedJavaObject(U* Object)
		: ObjRef{reinterpret_cast<T>(Object)}
	{
	}

	template <typename U> requires std::is_base_of_v<std::remove_pointer_t<U>, std::remove_pointer_t<T>> // FIXME needs ndk r28+, should be std::is_pointer_interconvertible_base_of_v
	explicit constexpr FScopedJavaObject(FScopedJavaObject<U>&& Object)
		: ObjRef{reinterpret_cast<T>(Object.Leak())}
	{
	}
	
	//UE_DEPRECATED(5.7, "Use FScopedJavaObject(T InObjRef) instead.")
	FScopedJavaObject(JNIEnv* InEnv, T InObjRef)
		: ObjRef{InObjRef}
	{
	}
	
	FScopedJavaObject(FScopedJavaObject&& Other)
		: ObjRef{std::exchange(Other.ObjRef, nullptr)}
	{
	}
	
	FScopedJavaObject(const FScopedJavaObject&) = delete;

	~FScopedJavaObject()
	{
		if (*this)
		{
			UE::Jni::Env->DeleteLocalRef(ObjRef);
		}
	}

	FScopedJavaObject& operator=(FScopedJavaObject&& Other)
	{
		Swap(ObjRef, Other.ObjRef);
		return *this;
	}
	
	FScopedJavaObject& operator=(const FScopedJavaObject&) = delete;

	[[nodiscard]] T Leak()
	{
		return std::exchange(ObjRef, nullptr);
	}

	// Returns the underlying JNI pointer
	[[nodiscard]] T operator*() const { return ObjRef; }
	
	operator bool() const
	{
		return ObjRef;
	}
	
private:
	T ObjRef = nullptr;
};

/**
 Helper function that allows template deduction on the java object type, for example:
 auto ScopeObject = NewScopedJavaObject(Env, JavaString);
 instead of FScopedJavaObject<jstring> ScopeObject(Env, JavaString);
 */
template <typename T>
//UE_DEPRECATED(5.7, "Use CTAD instead. Example: FScopedJavaObject{InObjRef}.")
CORE_API FScopedJavaObject<T> NewScopedJavaObject(JNIEnv* InEnv, const T& InObjRef)
{
	return FScopedJavaObject<T>(InEnv, InObjRef);
}

class FJavaHelper
{
public:
	// Converts the java string to FString and calls DeleteLocalRef on the passed-in java string reference
	static CORE_API FString FStringFromLocalRef(JNIEnv* Env, jstring JavaString);
	
	// Converts the java string to FString and calls DeleteGlobalRef on the passed-in java string reference
	static CORE_API FString FStringFromGlobalRef(JNIEnv* Env, jstring JavaString);
	
	// Converts the java string to FString, does NOT modify the passed-in java string reference
	static CORE_API FString FStringFromParam(JNIEnv* Env, jstring JavaString);
	
	// Converts FString into a Java string wrapped in FScopedJavaObject
	static CORE_API FScopedJavaObject<jstring> ToJavaString(JNIEnv* Env, const FString& UnrealString);

	// Converts a TArray<FStringView> into a Java string array wrapped in FScopedJavaObject. FStringView content is expected to be null terminated
	static CORE_API FScopedJavaObject<jobjectArray> ToJavaStringArray(JNIEnv* Env, const TArray<FStringView>& UnrealStrings);

	// Converts the java objectArray to an array of FStrings. jopbjectArray must be a String[] on the Java side
	static CORE_API TArray<FString> ObjectArrayToFStringTArray(JNIEnv* Env, jobjectArray ObjectArray);
};
#endif