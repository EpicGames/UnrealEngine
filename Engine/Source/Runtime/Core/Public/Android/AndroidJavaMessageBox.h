// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if USE_ANDROID_JNI

#include "AndroidJava.h"

// Wrapper for com/epicgames/unreal/MessageBox*.java.
class FJavaAndroidMessageBox : public FJavaClassObject
{
public:
	FJavaAndroidMessageBox();
	void SetCaption(const FString & Text);
	void SetText(const FString & Text);
	void AddButton(const FString & Text);
	void Clear();
	int32 Show();

private:
	static FName GetClassName();

	FJavaClassMethod SetCaptionMethod;
	FJavaClassMethod SetTextMethod;
	FJavaClassMethod AddButtonMethod;
	FJavaClassMethod ClearMethod;
	FJavaClassMethod ShowMethod;
};

#endif