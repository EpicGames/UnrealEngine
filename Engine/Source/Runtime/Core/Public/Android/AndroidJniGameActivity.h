// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if USE_ANDROID_JNI
#include "Android/AndroidJavaEnv.h"
#include "Android/AndroidJniGameControllerManager.h"

namespace UE::Jni::Android::Content
{
	struct FIntent: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "android/content/Intent";
	};
}

namespace UE::Jni::Android::View
{
	struct FSurface: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "android/view/Surface";
	};
}

namespace UE::Jni
{
	struct FGameActivity: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/GameActivity";

		static FGameActivity* Get();
#if USE_ANDROID_INPUT
		
		inline static TMember<FGameActivity*, FGameControllerManager*> gameControllerManager;

		inline static TMember<FGameActivity*, jboolean ()> createGameControllerManager;
#endif

		static void JNICALL nativeHandleNewIntentData(JNIEnv* env, jobject thiz, jstring JavaIntent);
		static void JNICALL nativeVirtualKeyboardVisible(JNIEnv* env, jobject thiz, jboolean bShown);
		static void JNICALL nativeVirtualKeyboardShown(JNIEnv* env, jobject thiz, jint left, jint top, jint right, jint bottom);
		static void JNICALL nativeVirtualKeyboardResult(JNIEnv* env, jobject thiz, jboolean update, jstring contents);
		static void JNICALL nativeVirtualKeyboardDismissed(JNIEnv* env, jobject thiz, jboolean update, jstring contents);
		static void JNICALL nativeVirtualKeyboardChanged(JNIEnv* env, jobject thiz, jstring contents);
		static void JNICALL nativeVirtualKeyboardSendKey(JNIEnv* env, jobject thiz, jint keyCode);
		static void JNICALL nativeVirtualKeyboardSendSelection(JNIEnv* jenv, jobject thiz, jint selStart, jint selEnd);
		static void JNICALL nativeSetObbFilePaths(JNIEnv* env, jobject thiz, jstring OBBMainFilePath, jstring OBBPatchFilePath, jstring OBBOverflow1FilePath, jstring OBBOverflow2FilePath);
		static void JNICALL nativeSetGlobalActivity(JNIEnv* env, jobject thiz, jboolean bUseExternalFilesDir, jboolean bPublicLogFiles, jstring internalFilePath, jstring externalFilePath, jboolean bOBBinAPK, jstring APKFilename);
		static jboolean JNICALL nativeIsShippingBuild(JNIEnv* env, jobject thiz);
		static void JNICALL nativeOnActivityResult(JNIEnv* env, jobject thiz, FGameActivity* activity, jint requestCode, jint resultCode, Android::Content::FIntent* data);
		static void JNICALL nativeHandleSensorEvents(JNIEnv* env, jobject thiz, Java::Lang::TArray<jfloat>* tilt, Java::Lang::TArray<jfloat>* rotation_rate, Java::Lang::TArray<jfloat>* gravity, Java::Lang::TArray<jfloat>* acceleration);
		static void JNICALL nativeOnThermalStatusChangedListener(JNIEnv* env, jobject thiz, jint Status);
		static void JNICALL nativeOnTrimMemory(JNIEnv* env, jobject thiz, jint MemoryTrimValue);
		static void JNICALL nativeOnOrientationChanged(JNIEnv* env, jobject thiz, jint orientation);
		static void JNICALL nativeConsoleCommand(JNIEnv* env, jobject thiz, jstring commandString);
		static void JNICALL nativeInitHMDs(JNIEnv* env, jobject thiz);
		static void JNICALL nativeSetAndroidVersionInformation(JNIEnv* env, jobject thiz, jstring androidVersion, jint targetSDKversion, jstring phoneMake, jstring phoneModel, jstring phoneBuildNumber, jstring osLanguage, jstring productName);
		static void JNICALL nativeOnInitialDownloadStarted(JNIEnv* env, jobject thiz);
		static void JNICALL nativeOnInitialDownloadCompleted(JNIEnv* env, jobject thiz);
		static void JNICALL nativeCrashContextSetStringKey(JNIEnv* env, jobject thiz, jstring JavaKey, jstring JavaValue);
		static void JNICALL nativeCrashContextSetBooleanKey(JNIEnv* env, jobject thiz, jstring JavaKey, jboolean JavaValue);
		static void JNICALL nativeCrashContextSetIntegerKey(JNIEnv* env, jobject thiz, jstring JavaKey, jint JavaValue);
		static void JNICALL nativeCrashContextSetFloatKey(JNIEnv* env, jobject thiz, jstring JavaKey, jfloat JavaValue);
		static void JNICALL nativeCrashContextSetDoubleKey(JNIEnv* env, jobject thiz, jstring JavaKey, jdouble JavaValue);
		static void JNICALL nativeSetObbInfo(JNIEnv* env, jobject thiz, jstring ProjectName, jstring PackageName, jint Version, jint PatchVersion, jstring AppType);
		static void JNICALL nativeSetAffinityInfo(JNIEnv* env, jobject thiz, jboolean bEnableAffinity, jint bigCoreMask, jint littleCoreMask);
		static void JNICALL nativeSetConfigRulesVariables(JNIEnv* env, jobject thiz, Java::Lang::TArray<jstring>* KeyValuePairs, Java::Lang::TArray<jbyte>* Utf8Vars);
		static void JNICALL nativeSetAndroidStartupState(JNIEnv* env, jobject thiz, jboolean bDebuggerAttached);
		static void JNICALL nativeNetworkChanged(JNIEnv* env, jobject thiz);
		static void JNICALL nativeResumeMainInit(JNIEnv* env, jobject thiz);
		static jint JNICALL nativeGetCPUFamily(JNIEnv* env, jobject thiz);
		static jboolean JNICALL nativeSupportsNEON(JNIEnv* env, jobject thiz);
		static void JNICALL nativeSetWindowInfo(JNIEnv* env, jobject thiz, jboolean bIsPortrait, jint DepthBufferPreference, jint PropagateAlpha);
		static void JNICALL nativeSetSurfaceViewInfo(JNIEnv* env, jobject thiz, jint width, jint height);
		static void JNICALL nativeSetSafezoneInfo(JNIEnv* env, jobject thiz, jboolean bIsPortrait, jfloat left, jfloat top, jfloat right, jfloat bottom);
		static void JNICALL nativeOnAudioFocusChanged(JNIEnv* env, jobject thiz, jint FocusChange);
#if USE_ANDROID_INPUT

		static constexpr FMember Members[]
		{
			UE_JNI_MEMBER(gameControllerManager),
			UE_JNI_MEMBER(createGameControllerManager)
		};
#endif
		
		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(nativeHandleNewIntentData),
			UE_JNI_NATIVE_METHOD(nativeVirtualKeyboardVisible),
			UE_JNI_NATIVE_METHOD(nativeVirtualKeyboardShown),
			UE_JNI_NATIVE_METHOD(nativeVirtualKeyboardResult),
			UE_JNI_NATIVE_METHOD(nativeVirtualKeyboardDismissed),
			UE_JNI_NATIVE_METHOD(nativeVirtualKeyboardChanged),
			UE_JNI_NATIVE_METHOD(nativeVirtualKeyboardSendKey),
			UE_JNI_NATIVE_METHOD(nativeVirtualKeyboardSendSelection),
			UE_JNI_NATIVE_METHOD(nativeSetObbFilePaths),
			UE_JNI_NATIVE_METHOD(nativeSetGlobalActivity),
			UE_JNI_NATIVE_METHOD(nativeIsShippingBuild),
			UE_JNI_NATIVE_METHOD(nativeOnActivityResult),
			UE_JNI_NATIVE_METHOD(nativeHandleSensorEvents),
			UE_JNI_NATIVE_METHOD(nativeOnThermalStatusChangedListener),
			UE_JNI_NATIVE_METHOD(nativeOnTrimMemory),
			UE_JNI_NATIVE_METHOD(nativeOnOrientationChanged),
			UE_JNI_NATIVE_METHOD(nativeConsoleCommand),
			UE_JNI_NATIVE_METHOD(nativeInitHMDs),
			UE_JNI_NATIVE_METHOD(nativeSetAndroidVersionInformation),
			UE_JNI_NATIVE_METHOD(nativeOnInitialDownloadStarted),
			UE_JNI_NATIVE_METHOD(nativeOnInitialDownloadCompleted),
			UE_JNI_NATIVE_METHOD(nativeCrashContextSetStringKey),
			UE_JNI_NATIVE_METHOD(nativeCrashContextSetBooleanKey),
			UE_JNI_NATIVE_METHOD(nativeCrashContextSetIntegerKey),
			UE_JNI_NATIVE_METHOD(nativeCrashContextSetFloatKey),
			UE_JNI_NATIVE_METHOD(nativeCrashContextSetDoubleKey),
			UE_JNI_NATIVE_METHOD(nativeSetObbInfo),
			UE_JNI_NATIVE_METHOD(nativeSetAffinityInfo),
			UE_JNI_NATIVE_METHOD(nativeSetConfigRulesVariables),
			UE_JNI_NATIVE_METHOD(nativeSetAndroidStartupState),
			UE_JNI_NATIVE_METHOD(nativeNetworkChanged),
			UE_JNI_NATIVE_METHOD(nativeResumeMainInit),
			UE_JNI_NATIVE_METHOD(nativeGetCPUFamily),
			UE_JNI_NATIVE_METHOD(nativeSupportsNEON),
			UE_JNI_NATIVE_METHOD(nativeSetWindowInfo),
			UE_JNI_NATIVE_METHOD(nativeSetSurfaceViewInfo),
			UE_JNI_NATIVE_METHOD(nativeSetSafezoneInfo),
			UE_JNI_NATIVE_METHOD(nativeOnAudioFocusChanged)
		};
	};
}

namespace UE::Jni::Asis
{
	struct FAsisGameActivity: FGameActivity
	{
		static constexpr FAnsiStringView ClassName = "com/epicgames/asis/AsisGameActivity";
	
		static jstring JNICALL nativeGetObbComment(JNIEnv* env, jclass clazz);
		static void JNICALL nativeMain(JNIEnv* env, jclass clazz, jstring projectModule);
		static void JNICALL nativeSetCommandline(JNIEnv* env, jclass clazz, jstring commandline);
		static void JNICALL nativeAppCommand(JNIEnv* env, jclass clazz, jint cmd);
	
		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(nativeGetObbComment),
			UE_JNI_NATIVE_METHOD(nativeMain),
			UE_JNI_NATIVE_METHOD(nativeSetCommandline),
			UE_JNI_NATIVE_METHOD(nativeAppCommand)
		};
#if USE_ANDROID_INPUT

	private:
		using FGameActivity::Members;
#endif
	};
}
#if USE_ANDROID_STANDALONE

namespace UE::Jni
{
	template struct TInitialize<Asis::FAsisGameActivity>;
}
#endif
#endif
