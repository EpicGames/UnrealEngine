// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if USE_ANDROID_JNI && USE_ANDROID_INPUT
#include "Android/AndroidJavaEnv.h"

namespace UE::Jni
{
	struct FGameControllerManager: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/GameControllerManager";
		
		inline static TMember<FGameControllerManager*, void ()> scanDevices;
		
		static void JNICALL nativeOnInputDeviceStateEvent(JNIEnv* env, jobject thiz, jint device_id, jint state, jint type);

		static constexpr FMember Members[]
		{
			UE_JNI_MEMBER(scanDevices)
		};
		
		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(nativeOnInputDeviceStateEvent)
		};
	};
	
	template struct TInitialize<FGameControllerManager>;
}
#endif
