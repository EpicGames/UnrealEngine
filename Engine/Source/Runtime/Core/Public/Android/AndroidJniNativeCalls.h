// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if USE_ANDROID_JNI
#include "Android/AndroidJavaEnv.h"

namespace UE::Jni
{
	struct FNativeCalls: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/NativeCalls";

		static void JNICALL WebViewVisible(JNIEnv* env, jclass clazz, jboolean bShown);
		static void JNICALL CallNativeToEmbedded(JNIEnv* env, jclass clazz, jstring InID, jint Priority, jstring InSubsystem, jstring InCommand, Java::Lang::TArray<jstring>* InParams, jstring InRoutingFunction);
		static void JNICALL SetNamedObject(JNIEnv* env, jclass clazz, jstring InName, jobject InObj);
		static void JNICALL KeepAwake(JNIEnv* env, jclass clazz, jstring InRequester, jboolean bIsForRendering);
		static void JNICALL AllowSleep(JNIEnv* env, jclass clazz, jstring InRequester);
		static void JNICALL UELogError(JNIEnv* env, jclass clazz, jstring InString);
		static void JNICALL UELogWarning(JNIEnv* env, jclass clazz, jstring InString);
		static void JNICALL UELogLog(JNIEnv* env, jclass clazz, jstring InString);
		static void JNICALL UELogVerbose(JNIEnv* env, jclass clazz, jstring InString);
		static void JNICALL ForwardNotification(JNIEnv* env, jclass clazz, jstring payload);
		static void JNICALL RouteServiceIntent(JNIEnv* env, jclass clazz, jstring InAction, jstring InPayload);
		static void JNICALL HandleCustomTouchEvent(JNIEnv* env, jclass clazz, jint deviceId, jint pointerId, jint action, jint soucre, jfloat x, jfloat y);
		static void JNICALL AllowJavaBackButtonEvent(JNIEnv* env, jclass clazz, jboolean allow);

		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(WebViewVisible),
			UE_JNI_NATIVE_METHOD(CallNativeToEmbedded),
			UE_JNI_NATIVE_METHOD(SetNamedObject),
			UE_JNI_NATIVE_METHOD(KeepAwake),
			UE_JNI_NATIVE_METHOD(AllowSleep),
			UE_JNI_NATIVE_METHOD(UELogError),
			UE_JNI_NATIVE_METHOD(UELogWarning),
			UE_JNI_NATIVE_METHOD(UELogLog),
			UE_JNI_NATIVE_METHOD(UELogVerbose),
			UE_JNI_NATIVE_METHOD(ForwardNotification),
			UE_JNI_NATIVE_METHOD(RouteServiceIntent),
			UE_JNI_NATIVE_METHOD(HandleCustomTouchEvent),
			UE_JNI_NATIVE_METHOD(AllowJavaBackButtonEvent)
		};
	};
	
	template struct TInitialize<FNativeCalls>;
}
#endif
