// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	AndroidPlatform.h: Setup for the Android platform
==================================================================================*/

#pragma once

#include "GenericPlatform/GenericPlatform.h"
#include "Clang/ClangPlatform.h"
#include "Misc/Build.h"

/** Define the android platform to be the active one **/
#define PLATFORM_ANDROID				1

/**
* Android specific types
**/
struct FAndroidTypes : public FGenericPlatformTypes
{
	//typedef unsigned int				DWORD;
	//typedef size_t					SIZE_T;
	//typedef decltype(NULL)			TYPE_OF_NULL;
	typedef char16_t					WIDECHAR;
	typedef WIDECHAR					TCHAR;
};

typedef FAndroidTypes FPlatformTypes;

#define ANDROID_MAX_PATH						PATH_MAX

// Base defines, must define these for the platform, there are no defaults
#define PLATFORM_DESKTOP				0
#define PLATFORM_CAN_SUPPORT_EDITORONLY_DATA	0

// Base defines, defaults are commented out

#define PLATFORM_LITTLE_ENDIAN						1
#define PLATFORM_SUPPORTS_PRAGMA_PACK				1
#define PLATFORM_USE_LS_SPEC_FOR_WIDECHAR			1
#define PLATFORM_HAS_BSD_TIME						1
#define PLATFORM_USE_PTHREADS						1
#define PLATFORM_MAX_FILEPATH_LENGTH_DEPRECATED		ANDROID_MAX_PATH
#define PLATFORM_SUPPORTS_TEXTURE_STREAMING			1
#define PLATFORM_REQUIRES_FILESERVER				1
#define PLATFORM_WCHAR_IS_4_BYTES					1
#define PLATFORM_TCHAR_IS_CHAR16					1
#define PLATFORM_HAS_NO_EPROCLIM					1
#define PLATFORM_USES_GLES							1
#define PLATFORM_BUILTIN_VERTEX_HALF_FLOAT			0
#define PLATFORM_HAS_BSD_IPV6_SOCKETS				1
#define PLATFORM_HAS_BSD_SOCKET_FEATURE_IOCTL		1
#define PLATFORM_HAS_BSD_SOCKET_FEATURE_MSG_DONTWAIT	1
#define PLATFORM_HAS_TOUCH_MAIN_SCREEN				1
#define PLATFORM_HAS_MULTITHREADED_PREMAIN			1
#define PLATFORM_SUPPORTS_STACK_SYMBOLS				1
#define PLATFORM_NUM_AUDIODECOMPRESSION_PRECACHE_BUFFERS 2
#define PLATFORM_USES_ANSI_STRING_FOR_EXTERNAL_PROFILING 1
#define PLATFORM_UI_HAS_MOBILE_SCROLLBARS			1
#define PLATFORM_UI_NEEDS_TOOLTIPS					0
#define PLATFORM_UI_NEEDS_FOCUS_OUTLINES			0
#define PLATFORM_SUPPORTS_EARLY_MOVIE_PLAYBACK		1 // movies will start before engine is initialized
#define PLATFORM_SUPPORTS_GEOMETRY_SHADERS			(RHI_RAYTRACING) // Currently required due to the way we offset ShaderStage::EStage
#define PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS	1
#define PLATFORM_USE_ANSI_POSIX_MALLOC				1
#define PLATFORM_USE_MINIMAL_HANG_DETECTION			1
#define PLATFORM_CODE_SECTION(Name)					__attribute__((section(Name)))
#define PLATFORM_ALLOW_ALLOCATIONS_IN_FASYNCWRITER_SERIALIZEBUFFERTOARCHIVE	0
#define PLATFORM_SUPPORTS_MIMALLOC					1
#define PLATFORM_STORES_DATA_IN_POINTER_HIGH_BITS	1

#define PLATFORM_RETURN_ADDRESS_FOR_CALLSTACKTRACING	PLATFORM_RETURN_ADDRESS

#define PLATFORM_GLOBAL_LOG_CATEGORY				LogAndroid

#define PLATFORM_ENABLE_VECTORINTRINSICS			1
#define PLATFORM_ENABLE_VECTORINTRINSICS_NEON		PLATFORM_ANDROID_ARM64

// Ensure we can use this builtin - seems to be present on Clang 9, GCC 11 and MSVC 19.26,
// but gives spurious "non-void function 'BitCast' should return a value" errors on some
// Mac and Android toolchains when building PCHs, so avoid those.
#undef PLATFORM_COMPILER_SUPPORTS_BUILTIN_BITCAST
#define PLATFORM_COMPILER_SUPPORTS_BUILTIN_BITCAST (__clang_major__ >= 13)

// some android platform overrides that sub-platforms can disable
#ifndef USE_ANDROID_JNI
	#define USE_ANDROID_JNI							1
#endif
#ifndef USE_ANDROID_AUDIO
	#define USE_ANDROID_AUDIO						1
#endif
#ifndef USE_ANDROID_FILE
	#define USE_ANDROID_FILE						1
#endif
#ifndef USE_ANDROID_LAUNCH
	#define USE_ANDROID_LAUNCH						1
#endif
#ifndef USE_ANDROID_INPUT
	#define USE_ANDROID_INPUT						1
#endif
#ifndef USE_ANDROID_EVENTS
	#define USE_ANDROID_EVENTS						1
#endif
#ifndef USE_ANDROID_STANDALONE
	#define USE_ANDROID_STANDALONE					0
#endif
#ifndef USE_ANDROID_ALTERNATIVE_SUSPEND
	#define USE_ANDROID_ALTERNATIVE_SUSPEND			0
#endif


#if (!USE_ANDROID_STANDALONE && UE_BUILD_DEBUG) || (USE_ANDROID_STANDALONE && !UE_BUILD_SHIPPING)
	// M is the scope for the logging such as LogAndroid, STANDALONE_DEBUG_LOGf should be used when using formatted arguments.
#	define STANDALONE_DEBUG_LOG(M, ...)  FPlatformMisc::LowLevelOutputDebugStringf(TEXT(#M " : "),##__VA_ARGS__);
#	define STANDALONE_DEBUG_LOGf(M, ...)  FPlatformMisc::LowLevelOutputDebugStringf(TEXT(#M " : ") __VA_ARGS__);

#else
#	define STANDALONE_DEBUG_LOG(...)  
#	define STANDALONE_DEBUG_LOGf(...)  
#endif

#define ANDROID_GAMEACTIVITY_BASE_CLASSPATH			"com/epicgames/unreal/GameActivity"

#if USE_ANDROID_STANDALONE
#	define ANDROID_GAMEACTIVITY_CLASSPATH	"com/epicgames/asis/AsisGameActivity"
#else
#	define ANDROID_GAMEACTIVITY_CLASSPATH	"com/epicgames/unreal/GameActivity"
#endif



// Enable to set thread nice values when setting runnable thread priorities
#ifndef ANDROID_USE_NICE_VALUE_THREADPRIORITY
	#define ANDROID_USE_NICE_VALUE_THREADPRIORITY 0
#endif

// Function type macros.
#define VARARGS													/* Functions with variable arguments */
#define CDECL													/* Standard C function */
#define STDCALL													/* Standard calling convention */
#if UE_BUILD_DEBUG 
	#define FORCEINLINE	inline									/* Easier to debug */
#else
	#define FORCEINLINE inline __attribute__ ((always_inline))	/* Force code to be inline */
#endif
#define FORCENOINLINE __attribute__((noinline))					/* Force code to NOT be inline */

#define FUNCTION_CHECK_RETURN_END __attribute__ ((warn_unused_result))	/* Warn that callers should not ignore the return value. */
#define FUNCTION_NO_RETURN_END __attribute__ ((noreturn))				/* Indicate that the function never returns. */

// Optimization macros
#define PRAGMA_DISABLE_OPTIMIZATION_ACTUAL _Pragma("clang optimize off")
#define PRAGMA_ENABLE_OPTIMIZATION_ACTUAL _Pragma("clang optimize on")

// Disable optimization of a specific function
#define DISABLE_FUNCTION_OPTIMIZATION	__attribute__((optnone))

#define ABSTRACT abstract

// DLL export and import definitions
#define DLLEXPORT			__attribute__((visibility("default")))
#define DLLIMPORT			__attribute__((visibility("default")))
#define JNI_METHOD			__attribute__ ((visibility ("default"))) extern "C"

// Alignment.
#define GCC_PACK(n)			__attribute__((packed,aligned(n)))
#define GCC_ALIGN(n)		__attribute__((aligned(n)))

// operator new/delete operators
// As of 10.9 we need to use _NOEXCEPT & cxx_noexcept compatible definitions
#if __has_feature(cxx_noexcept)
#define OPERATOR_NEW_THROW_SPEC
#else
#define OPERATOR_NEW_THROW_SPEC throw (std::bad_alloc)
#endif
#define OPERATOR_DELETE_THROW_SPEC noexcept
#define OPERATOR_NEW_NOTHROW_SPEC  noexcept
#define OPERATOR_DELETE_NOTHROW_SPEC  noexcept
