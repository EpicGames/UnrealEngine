// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
AndroidAffinity.h: Android affinity profile masks definitions.
==============================================================================================*/

#pragma once

#include "GenericPlatform/GenericPlatformAffinity.h"
#include "AndroidPlatform.h"

class FAndroidAffinity : public FGenericPlatformAffinity
{
private:
	static uint64 GetLittleCoreMask();
	const static uint64 AllCores = 0xFFFFFFFFFFFFFFFF;
public:
	static const uint64 GetMainGameMask()
	{
		return GameThreadMask;
	}

	static const uint64 GetRenderingThreadMask()
	{
		return RenderingThreadMask;
	}

	static const uint64 GetRHIThreadMask()
	{
		return AllCores;
	}

	static const uint64 GetRTHeartBeatMask()
	{
		return GetLittleCoreMask();
	}

	static const uint64 GetPoolThreadMask()
	{
#if ANDROID_USE_NICE_VALUE_THREADPRIORITY
		return AllCores;
#else
		return GetLittleCoreMask();
#endif
	}

	static const uint64 GetTaskGraphThreadMask()
	{
		return AllCores;
	}

	static const uint64 GetAudioRenderThreadMask()
	{
		return GetLittleCoreMask();
	}

	static const uint64 GetTaskGraphBackgroundTaskMask()
	{
		return GetLittleCoreMask();
	}

	static const uint64 GetTaskGraphHighPriorityTaskMask()
	{
		return AllCores;
	}

	static const uint64 GetAsyncLoadingThreadMask()
	{
		return AllCores;
	}

	static EThreadPriority GetRenderingThreadPriority()
	{
		return TPri_SlightlyBelowNormal;
	}

	static EThreadPriority GetRHIThreadPriority()
	{
		return TPri_Normal;
	}

public:
	static uint64 GameThreadMask;
	static uint64 RenderingThreadMask;
};

typedef FAndroidAffinity FPlatformAffinityBase;
