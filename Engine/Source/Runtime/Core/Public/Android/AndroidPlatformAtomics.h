// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	AndroidPlatformAtomics.h: Android platform Atomics functions
==============================================================================================*/

#pragma once

#include "Clang/ClangPlatformAtomics.h"

typedef FClangPlatformAtomics FPlatformAtomics;
