// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Make certain warnings always be warnings, even despite -Werror.
// Rationale: we don't want to suppress those as there are plans to address them (e.g. UE-12341), but breaking builds due to these warnings is very expensive
// since they cannot be caught by all compilers that we support. They are deemed to be relatively safe to be ignored, at least until all SDKs/toolchains start supporting them.
#pragma clang diagnostic warning "-Wparentheses-equality"

#define PRAGMA_DISABLE_OPTIMIZATION_ACTUAL _Pragma("clang optimize off")
#define PRAGMA_ENABLE_OPTIMIZATION_ACTUAL  _Pragma("clang optimize on")

#include "Clang/ClangPlatformCompilerPreSetup.h"