// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	AndroidCompilerSetup.h: pragmas, version checks and other things for the Android compiler
==============================================================================================*/

#pragma once

// The register keyword is deprecated
#define register