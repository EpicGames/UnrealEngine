// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Containers/Map.h"

struct FAndroidCrashContext : public FGenericCrashContext
{
	/** Signal number */
	int32 Signal;

	/** Id of a thread that crashed */
	uint32 CrashingThreadId;
	
	/** Additional signal info */
	siginfo* Info;
	
	/** Thread context */
	void* Context;

	CORE_API FAndroidCrashContext(ECrashContextType InType, const TCHAR* InErrorMessage);

	~FAndroidCrashContext() = default;

	/**
	 * Inits the crash context from data provided by a signal handler.
	 *
	 * @param InSignal number (SIGSEGV, etc)
	 * @param InInfo additional info (e.g. address we tried to read, etc)
	 * @param InContext thread context
	 */
	void InitFromSignal(int32 InSignal, siginfo* InInfo, void* InContext, uint32 InCrashingThreadId)
	{
		Signal = InSignal;
		CrashingThreadId = InCrashingThreadId;
		Info = InInfo;
		Context = InContext;
	}

	CORE_API virtual void GetPortableCallStack(const uint64* StackFrames, int32 NumStackFrames, TArray<FCrashStackFrame>& OutCallStack) const override;
	CORE_API virtual void AddPlatformSpecificProperties() const override;
	CORE_API virtual bool ShouldIncludeInPortableCallStackHash(const FCrashStackFrame& Frame) const override;

	CORE_API void CaptureCrashInfo();
	CORE_API void StoreCrashInfo(bool bWriteLog) const;

	static const int32 CrashReportMaxPathSize = 512;

	static CORE_API void Initialize();
	// Returns the report directory used for this crash context.
	FString GetCurrentReportDirectoryPath() const {	return FString(ReportDirectory); }

	// Returns the main crash directory for this session. This will not be correct for non-fatal reports.
	static CORE_API const FString GetGlobalCrashDirectoryPath();
	// Fills DirectoryNameOUT with the global crash directory for a fatal crash this session. This will not be correct for non-fatal reports.
	static CORE_API void GetGlobalCrashDirectoryPath(char(&DirectoryNameOUT)[CrashReportMaxPathSize]);

	CORE_API void AddAndroidCrashProperty(const FString& Key, const FString& Value);

	CORE_API void SetOverrideCallstack(const FString& OverrideCallstackIN);

	// generate an absolute path to a crash report folder.
	static CORE_API void GenerateReportDirectoryName(char(&DirectoryNameOUT)[CrashReportMaxPathSize]);

	// expects externally allocated memory for all threads found in FThreadManager + main thread
	CORE_API void DumpAllThreadCallstacks(FAsyncThreadBackTrace* BackTrace, int NumThreads) const;

	/** Async-safe ItoA */
	static CORE_API const ANSICHAR* ItoANSI(uint64 Val, uint64 Base, uint32 Len = 0);

	// temporary accessor to allow overriding of the portable callstack.
	TArray<FCrashStackFrame>& GetPortableCallstack_TEMP() { return CallStack; }

protected:

	/** Allow platform implementations to provide a callstack property. Primarily used when non-native code triggers a crash. */
	CORE_API virtual const TCHAR* GetCallstackProperty() const;

private:
	FString OverrideCallstack;

	TMap<FString, FString> AdditionalProperties;

	// The path used by this instance to store the report.
	char ReportDirectory[CrashReportMaxPathSize];
};

struct FAndroidMemoryWarningContext : public FGenericMemoryWarningContext
{
	FAndroidMemoryWarningContext() : LastTrimMemoryState(FAndroidPlatformMemory::ETrimValues::Unknown) {}

	// value last recorded from java side's OnTrimMemory.
	FAndroidPlatformMemory::ETrimValues LastTrimMemoryState;
};

typedef FAndroidCrashContext FPlatformCrashContext;
