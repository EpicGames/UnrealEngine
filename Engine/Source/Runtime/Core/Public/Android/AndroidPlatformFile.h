// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	AndroidFile.h: Android platform File functions
==============================================================================================*/

#pragma once

#include "GenericPlatform/GenericPlatformFile.h"
#if USE_ANDROID_JNI
#include <jni.h>
#endif

/**
	Android File I/O implementation with additional utilities to deal
	with Java side access.
**/
class IAndroidPlatformFile : public IPhysicalPlatformFile
{
public:
	// Methods that expose an argument for allowing Android Assets (i.e. `assets` directory in Gradle project) to be considered.  By default, they are
	// not, because Asset stat can be very slow on some devices.
	virtual bool FileExists(const TCHAR* Filename, bool bAllowAssets) = 0;
	virtual int64 FileSize(const TCHAR* Filename, bool bAllowAssets) = 0;
	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory, bool bAllowAssets) = 0;
	virtual bool DirectoryExists(const TCHAR* Directory, bool bAllowAssets) = 0;
	virtual bool IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor, bool bAllowAssets) = 0;
	virtual bool IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor, bool bAllowAssets) = 0;
	// Using statements avoid a compiler warning - this says, "yes I am intending to add virtual overloads in a subclass, no it's not a mistake".
	using IPlatformFile::FileExists;
	using IPlatformFile::FileSize;
	using IPlatformFile::GetStatData;
	using IPlatformFile::DirectoryExists;
	using IPlatformFile::IterateDirectory;
	using IPlatformFile::IterateDirectoryStat;

	static CORE_API IAndroidPlatformFile & GetPlatformPhysical();

#if USE_ANDROID_FILE
	/**
	 * Get the directory path to write log files to.
	 * This is /temp0 in shipping, or a path inside /data for other configs.
	 */
	static CORE_API const FString* GetOverrideLogDirectory();
#endif

#if USE_ANDROID_JNI
	// Get the android.content.res.AssetManager that Java code
	// should use to open APK assets.
	virtual jobject GetAssetManager() = 0;
#endif

	// Get detailed information for a file that
	// we can hand to other Android media classes for access.

	// Is file embedded as an asset in the APK?
	virtual bool IsAsset(const TCHAR* Filename) = 0;

	// Offset within file or asset where its data starts.
	// Note, offsets for assets is relative to complete APK file
	// and matches what is returned by AssetFileDescriptor.getStartOffset().
	virtual int64 FileStartOffset(const TCHAR* Filename) = 0;

	// Get the root, i.e. underlying, path for the file. This
	// can be any of: a resolved file path, an OBB path, an
	// asset path.
	virtual FString FileRootPath(const TCHAR* Filename) = 0;

	CORE_API virtual FString ConvertToAbsolutePathForExternalAppForRead(const TCHAR* Filename) override;
	CORE_API virtual FString ConvertToAbsolutePathForExternalAppForWrite(const TCHAR* Filename) override;
};
