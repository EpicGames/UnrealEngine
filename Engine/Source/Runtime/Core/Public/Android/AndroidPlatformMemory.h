// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	AndroidPlatformMemory.h: Android platform memory functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformMemory.h"

/**
 *	Android implementation of the FGenericPlatformMemoryStats.
 */
struct FPlatformMemoryStats : public FGenericPlatformMemoryStats
{
	FGenericPlatformMemoryStats::EMemoryPressureStatus GetMemoryPressureStatus() const;
	uint64 VMSwap = 0;
	uint64 VMRss = 0;
	uint64 RssFile = 0;
};

/**
* Android implementation of the memory OS functions
**/
struct FAndroidPlatformMemory : public FGenericPlatformMemory
{
	//~ Begin FGenericPlatformMemory Interface
	static CORE_API void Init();
	static CORE_API FPlatformMemoryStats GetStats();
	static CORE_API uint64 GetMemoryUsedFast();
	static CORE_API const FPlatformMemoryConstants& GetConstants();
	static CORE_API EPlatformMemorySizeBucket GetMemorySizeBucket();
	static CORE_API FMalloc* BaseAllocator();
	static CORE_API void* BinnedAllocFromOS(SIZE_T Size);
	static CORE_API void BinnedFreeToOS(void* Ptr, SIZE_T Size);

	class FPlatformVirtualMemoryBlock : public FBasicVirtualMemoryBlock
	{
	public:

		FPlatformVirtualMemoryBlock()
		{
		}

		FPlatformVirtualMemoryBlock(void *InPtr, uint32 InVMSizeDivVirtualSizeAlignment)
			: FBasicVirtualMemoryBlock(InPtr, InVMSizeDivVirtualSizeAlignment)
		{
		}
		FPlatformVirtualMemoryBlock(const FPlatformVirtualMemoryBlock& Other) = default;
		FPlatformVirtualMemoryBlock& operator=(const FPlatformVirtualMemoryBlock& Other) = default;

		void Commit(size_t InOffset, size_t InSize);
		void Decommit(size_t InOffset, size_t InSize);
		void FreeVirtual();

		UE_FORCEINLINE_HINT void CommitByPtr(void *InPtr, size_t InSize)
		{
			Commit(size_t(((uint8*)InPtr) - ((uint8*)Ptr)), InSize);
		}

		UE_FORCEINLINE_HINT void DecommitByPtr(void *InPtr, size_t InSize)
		{
			Decommit(size_t(((uint8*)InPtr) - ((uint8*)Ptr)), InSize);
		}

		UE_FORCEINLINE_HINT void Commit()
		{
			Commit(0, GetActualSize());
		}

		UE_FORCEINLINE_HINT void Decommit()
		{
			Decommit(0, GetActualSize());
		}

		UE_FORCEINLINE_HINT size_t GetActualSize() const
		{
			return VMSizeDivVirtualSizeAlignment * GetVirtualSizeAlignment();
		}

		static FPlatformVirtualMemoryBlock AllocateVirtual(size_t Size, size_t InAlignment = FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment());
		static size_t GetCommitAlignment();
		static size_t GetVirtualSizeAlignment();
	};

	static CORE_API bool GetLLMAllocFunctions(void*(*&OutAllocFunction)(size_t), void(*&OutFreeFunction)(void*, size_t), int32& OutAlignment);
	//~ End FGenericPlatformMemory Interface
	
	// ANDROID ONLY:
	enum class ETrimValues
	{
		Unknown = -1,
		Complete = 80,			// the process is nearing the end of the background LRU list, and if more memory isn't found soon it will be killed.
		Moderate = 60,			// the process is around the middle of the background LRU list
		Background = 40,		// the process has gone on to the background LRU list. 
		UI_Hidden = 20,			// the process had been showing a user interface, and is no longer doing so...
		Running_Critical = 15,	// the process is not an expendable background process, but the device is running extremely low on memory and is about to not be able to keep any background processes running..
		Running_Low = 10,		// the process is not an expendable background process, but the device is running low on memory.
		Running_Moderate = 5	// the process is not an expendable background process, but the device is running moderately low on memory.
	};
	// called when OS (via JNI) reports memory trouble, triggers MemoryWarningHandler callback on game thread if set.
	enum class EOSMemoryStatusCategory { OSTrim };
	static CORE_API void UpdateOSMemoryStatus(EOSMemoryStatusCategory OSMemoryStatusCategory, int value);

	static void AndroidMemoryProbeStart();
	static void AndroidMemoryProbeStop();
	static void AndroidMemoryProbeTick();
};

typedef FAndroidPlatformMemory FPlatformMemory;



