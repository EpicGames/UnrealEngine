// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	AndroidMisc.h: Android platform misc functions
==============================================================================================*/

#pragma once
#include "CoreTypes.h"
#include "Android/AndroidSystemIncludes.h"
#include "GenericPlatform/GenericPlatformMisc.h"

template <typename FuncType>
class TFunction;


#define PLATFORM_BREAK()	raise(SIGTRAP)

#define UE_DEBUG_BREAK_IMPL()	PLATFORM_BREAK()

#define ANDROID_HAS_RTSIGNALS PLATFORM_USED_NDK_VERSION_INTEGER >= 21

#ifndef ANDROID_PROFILING_ENABLED
#define ANDROID_PROFILING_ENABLED !(UE_BUILD_SHIPPING)
#endif

enum class ECrashContextType;

enum class EAndroidAudioFocusChange
{
	LossTransientCanDuck = -3,
	LossTransient = -2,
	Loss = -1,
	None = 0,
	Gain = 1
};

/**
 * Android implementation of the misc OS functions
 */
struct FAndroidMisc : public FGenericPlatformMisc
{
	static CORE_API void RequestExit( bool Force, const TCHAR* CallSite = nullptr);
	static CORE_API bool RestartApplication();
	static CORE_API void LocalPrint(const TCHAR *Message);
	static bool IsLocalPrintThreadSafe() { return true; }
	static CORE_API void PlatformPreInit();
	static CORE_API void PlatformInit();
	static CORE_API void PlatformTearDown();
	static CORE_API void PlatformHandleSplashScreen(bool ShowSplashScreen);
    static EDeviceScreenOrientation GetDeviceOrientation() { return DeviceOrientation; }
	static CORE_API void SetAllowedDeviceOrientation(EDeviceScreenOrientation NewAllowedDeviceOrientation);

	// Change this to an Enum with Always allow, allow and deny
	static CORE_API void SetCellularPreference(int32 Value);
	static CORE_API int32 GetCellularPreference();
    
	UE_FORCEINLINE_HINT static int32 GetMaxPathLength()
	{
		return ANDROID_MAX_PATH;
	}

	static CORE_API FString GetEnvironmentVariable(const TCHAR* VariableName);
	static CORE_API const TCHAR* GetSystemErrorMessage(TCHAR* OutBuffer, int32 BufferCount, int32 Error);
	static CORE_API uint32 GetLastError();
	static CORE_API EAppReturnType::Type MessageBoxExt( EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption );
	static CORE_API bool UseRenderThread();
	static CORE_API bool HasPlatformFeature(const TCHAR* FeatureName);
	static CORE_API bool SupportsES30();
	static CORE_API bool SetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, const FString& InValue);
	static CORE_API bool GetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, FString& OutValue);
	static CORE_API bool DeleteStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName);
	static CORE_API bool DeleteStoredSection(const FString& InStoreId, const FString& InSectionName);

	static CORE_API FString GetBillingConfigCountryCode();
	static CORE_API bool GetIsAlternativeBillingOnlyAvailable();

public:

	static CORE_API bool AllowThreadHeartBeat();

	struct FCPUStatTime{
		uint64_t			TotalTime;
		uint64_t			UserTime;
		uint64_t			NiceTime;
		uint64_t			SystemTime;
		uint64_t			SoftIRQTime;
		uint64_t			IRQTime;
		uint64_t			IdleTime;
		uint64_t			IOWaitTime;
	};

	struct FCPUState
	{
		const static int32			MaxSupportedCores = 16; //Core count 16 is maximum for now
		int32						CoreCount;
		int32						ActivatedCoreCount;
		ANSICHAR					Name[6];
		FAndroidMisc::FCPUStatTime	CurrentUsage[MaxSupportedCores]; 
		FAndroidMisc::FCPUStatTime	PreviousUsage[MaxSupportedCores];
		int32						Status[MaxSupportedCores];
		double						Utilization[MaxSupportedCores];
		double						AverageUtilization;
		
	};

	static CORE_API FCPUState& GetCPUState();
	static CORE_API int32 NumberOfCores();
	static CORE_API int32 NumberOfCoresIncludingHyperthreads();
	static CORE_API bool SupportsLocalCaching();
	static CORE_API void CreateGuid(struct FGuid& Result);
	static CORE_API void SetCrashHandler(void (* CrashHandler)(const FGenericCrashContext& Context));
	// NOTE: THIS FUNCTION IS DEFINED IN ANDROIDOPENGL.CPP
	static CORE_API void GetValidTargetPlatforms(class TArray<class FString>& TargetPlatformNames);
	static CORE_API bool GetUseVirtualJoysticks();
	static CORE_API bool SupportsTouchInput();
	static const TCHAR* GetDefaultDeviceProfileName();
	static CORE_API bool GetVolumeButtonsHandledBySystem();
	static CORE_API void SetVolumeButtonsHandledBySystem(bool enabled);
	// Returns current volume, 0-15
	static CORE_API int GetVolumeState(double* OutTimeOfChangeInSec = nullptr);

	static CORE_API int32 GetDeviceVolume();

#if USE_ANDROID_FILE
	static CORE_API const TCHAR* GamePersistentDownloadDir();
	static CORE_API FString GetLoginId();
#endif
#if USE_ANDROID_JNI
	static CORE_API FString GetDeviceId();
	static CORE_API FString GetUniqueAdvertisingId();
#endif
	static CORE_API FString GetCPUVendor();
	static CORE_API FString GetCPUBrand();
	static CORE_API FString GetCPUChipset();
	static CORE_API FString GetPrimaryGPUBrand();
	static CORE_API void GetOSVersions(FString& out_OSVersionLabel, FString& out_OSSubVersionLabel);
	static CORE_API bool GetDiskTotalAndFreeSpace(const FString& InPath, uint64& TotalNumberOfBytes, uint64& NumberOfFreeBytes);
	
	enum EBatteryState
	{
		BATTERY_STATE_UNKNOWN = 1,
		BATTERY_STATE_CHARGING,
		BATTERY_STATE_DISCHARGING,
		BATTERY_STATE_NOT_CHARGING,
		BATTERY_STATE_FULL
	};
	struct FBatteryState
	{
		FAndroidMisc::EBatteryState	State;
		int							Level;          // in range [0,100]
		float						Temperature;    // in degrees of Celsius
	};

	static CORE_API FBatteryState GetBatteryState();
	static CORE_API int GetBatteryLevel();
	static CORE_API bool IsRunningOnBattery();
	static CORE_API bool IsInLowPowerMode();
	static CORE_API float GetDeviceTemperatureLevel();
	static CORE_API float GetDeviceTemperature();
	static CORE_API EDeviceThermalState GetDeviceThermalState();
	static CORE_API bool AreHeadPhonesPluggedIn();
	static CORE_API ENetworkConnectionType GetNetworkConnectionType();
#if USE_ANDROID_JNI
	static CORE_API bool HasActiveWiFiConnection();
#endif

	static CORE_API void RegisterForRemoteNotifications();
	static CORE_API void UnregisterForRemoteNotifications();
	static CORE_API bool IsAllowedRemoteNotifications();
	
	static CORE_API FDelegateHandle RegisterAudioFocusChangedEventListener(const FOnAudioFocusChangedDelegate Callback);
	static CORE_API void UnregisterAudioFocusChangedEventListener(const FDelegateHandle DelegateHandle);

	static CORE_API void HandleNewIntentUri(const FString& IntentUri);
	static CORE_API FString GetPendingActivationProtocol();

	/** @return Memory representing a true type or open type font provided by the platform as a default font for unreal to consume; empty array if the default font failed to load. */
	static CORE_API TArray<uint8> GetSystemFontBytes();

	static CORE_API IPlatformChunkInstall* GetPlatformChunkInstall();

	static CORE_API void PrepareMobileHaptics(EMobileHapticsType Type);
	static CORE_API void TriggerMobileHaptics();
	static CORE_API void ReleaseMobileHaptics();
	static CORE_API void ShareURL(const FString& URL, const FText& Description, int32 LocationHintX, int32 LocationHintY);

	static CORE_API FString LoadTextFileFromPlatformPackage(const FString& RelativePath);
	static CORE_API bool FileExistsInPlatformPackage(const FString& RelativePath);

	// ANDROID ONLY:
	static CORE_API void SetVersionInfo(FString AndroidVersion, int32 InTargetSDKVersion, FString DeviceMake, FString DeviceModel, FString DeviceBuildNumber, FString OSLanguage, FString ProductName);
	static CORE_API const FString GetAndroidVersion();
	static CORE_API int32 GetAndroidMajorVersion();
	static CORE_API int32 GetTargetSDKVersion();
	static CORE_API const FString GetDeviceMake();
	static CORE_API const FString GetDeviceModel();
	static CORE_API const FString GetOSLanguage();
	static CORE_API const FString GetProductName(); // returns the product name, if available. e.g. 'Galaxy Tab S8' or empty string.
	static CORE_API const FString GetDeviceBuildNumber();
	static CORE_API const FString GetProjectVersion();
	static CORE_API FString GetDefaultLocale();
	static CORE_API FString GetGPUFamily();
	static CORE_API FString GetGLVersion();
	static CORE_API bool SupportsFloatingPointRenderTargets();
	static CORE_API bool SupportsShaderFramebufferFetch();
	static CORE_API bool SupportsShaderIOBlocks();
#if USE_ANDROID_JNI
	static CORE_API int GetAndroidBuildVersion();
#endif
	static CORE_API bool IsSupportedAndroidDevice();
	static CORE_API void SetForceUnsupported(bool bInOverride);
	
	static CORE_API const TMap<FString, FString>& GetConfigRuleVars();

	// return UTF8 versions of the configrules
	static CORE_API const FUtf8StringView GetConfigRulesVariableAsUtf8(const FUtf8StringView Key);

	/* HasVulkanDriverSupport
	 * @return true if this Android device supports a Vulkan API Unreal could use
	 */
	static CORE_API bool HasVulkanDriverSupport();

	/* IsVulkanAvailable
	 * @return	true if there is driver support, we have an RHI, we are packaged with Vulkan support,
	 *			and not we are not forcing GLES with a command line switch
	 */
	static CORE_API bool IsVulkanAvailable();
	static CORE_API bool IsDesktopVulkanAvailable();
	
	/* ShouldUseVulkan
	 * @return true if Vulkan is available, and not disabled by device profile cvar
	 */
	static CORE_API bool ShouldUseVulkan();
	static CORE_API bool ShouldUseDesktopVulkan();
	static CORE_API FString GetVulkanVersion();
	static CORE_API FString GetVKQualityRecommendation();
	static CORE_API FString GetVulkanDeviceNameString();

	static CORE_API bool IsExternalMemoryAndroidHardwareBufferExtensionLoaded();
	typedef TFunction<void(void* NewNativeHandle)> ReInitWindowCallbackType;
	static CORE_API ReInitWindowCallbackType GetOnReInitWindowCallback();
	static CORE_API void SetOnReInitWindowCallback(ReInitWindowCallbackType InOnReInitWindowCallback);
	typedef TFunction<void()> ReleaseWindowCallbackType;
	static CORE_API ReleaseWindowCallbackType GetOnReleaseWindowCallback();
	static CORE_API void SetOnReleaseWindowCallback(ReleaseWindowCallbackType InOnReleaseWindowCallback);
	static CORE_API FString GetOSVersion();
	static bool GetOverrideResolution(int32 &ResX, int32& ResY) { return false; }
	typedef TFunction<void()> OnPauseCallBackType;
	static CORE_API OnPauseCallBackType GetOnPauseCallback();
	static CORE_API void SetOnPauseCallback(OnPauseCallBackType InOnPauseCallback);
	static CORE_API void TriggerCrashHandler(ECrashContextType InType, const TCHAR* InErrorMessage, const TCHAR* OverrideCallstack = nullptr);

	// To help track down issues with failing crash handler.
	static CORE_API FString GetFatalSignalMessage(int Signal, siginfo* Info);
	static CORE_API void OverrideFatalSignalHandler(void (*FatalSignalHandlerOverrideFunc)(int Signal, struct siginfo* Info, void* Context, uint32 CrashingThreadId));
	// To help track down issues with failing crash handler.

	static CORE_API bool IsInSignalHandler();

#if !UE_BUILD_SHIPPING
	static CORE_API bool IsDebuggerPresent();
#endif

	UE_FORCEINLINE_HINT static void MemoryBarrier()
	{
		__sync_synchronize();
	}

	// If this true the RHI will not be blocked while the app is in the background. Instead the RHI will detach the window and any further draw operations will be made to an offscreen surface.
	// false, standard blocking background behavior will be used...
	static CORE_API bool UseNewWindowBehavior();

#if STATS || ENABLE_STATNAMEDEVENTS
	/** Override to add Android features when stats are on, otherwise use version on FGenericPlatformMisc. */
	static CORE_API void BeginNamedEvent(const struct FColor& Color, const TCHAR* Text);
	static CORE_API void BeginNamedEvent(const struct FColor& Color, const ANSICHAR* Text);
	static CORE_API void EndNamedEvent();
#endif

#if (STATS || ENABLE_STATNAMEDEVENTS)
	static CORE_API int32 TraceMarkerFileDescriptor;
#endif
	
	// run time compatibility information
	static CORE_API FString AndroidVersion; // version of android we are running eg "4.0.4"
	static CORE_API int32 AndroidMajorVersion; // integer major version of Android we are running, eg 10
	static CORE_API int32 TargetSDKVersion; // Target SDK version, eg 29.
	static CORE_API FString DeviceMake; // make of the device we are running on eg. "samsung"
	static CORE_API FString DeviceModel; // model of the device we are running on eg "SAMSUNG-SGH-I437"
	static CORE_API FString DeviceBuildNumber; // platform image build number of device "R16NW.G960NKSU1ARD6"
	static CORE_API FString OSLanguage; // language code the device is set to
	static CORE_API FString ProductName; // Product name, if available. e.g. 'Galaxy Tab S8' or empty string.

	// Build version of Android, i.e. API level.
	static CORE_API int32 AndroidBuildVersion;

	// Key/Value pair variables from the optional configuration.txt
	static CORE_API TMap<FString, FString> ConfigRulesVariables;

	static CORE_API bool VolumeButtonsHandledBySystem;

	static CORE_API bool bNeedsRestartAfterPSOPrecompile;

	enum class ECoreFrequencyProperty
	{
		CurrentFrequency,
		MaxFrequency,
		MinFrequency,
	};

	static CORE_API uint32 GetCoreFrequency(int32 CoreIndex, ECoreFrequencyProperty CoreFrequencyProperty);

	// Returns CPU temperature read from one of the configurable CPU sensors via android.CPUThermalSensorFilePath CVar or AndroidEngine.ini, [ThermalSensors] section.
	// Doesn't guarantee to work on all devices. Some devices require root access rights to read sensors information, in that case 0.0 will be returned
	static CORE_API float GetCPUTemperature();

	static CORE_API void UpdateDeviceOrientation();

	static void SaveDeviceOrientation(EDeviceScreenOrientation NewDeviceOrentation) { DeviceOrientation = NewDeviceOrentation; }

	// Window access is locked by the game thread before preinit and unlocked here after RHIInit
	static CORE_API void UnlockAndroidWindow();
	
	static CORE_API TArray<int32> GetSupportedNativeDisplayRefreshRates();

	static CORE_API bool SetNativeDisplayRefreshRate(int32 RefreshRate);

	static CORE_API int32 GetNativeDisplayRefreshRate();

	/**
	 * Returns whether or not a 16 bit index buffer should be promoted to 32 bit on load, needed for some Android devices
	 */
	static CORE_API bool Expand16BitIndicesTo32BitOnLoad();

	/**
	 * Will return true if we wish to propagate the alpha to the backbuffer
	 */
	static CORE_API int GetMobilePropagateAlphaSetting();

	static CORE_API bool SupportsBackbufferSampling();

	static CORE_API void SetMemoryWarningHandler(void (*Handler)(const FGenericMemoryWarningContext& Context));
	static CORE_API bool HasMemoryWarningHandler();

	// Android specific requesting of exit, *ONLY* use this function in signal handling code. Otherwise normal RequestExit functions
	static CORE_API void NonReentrantRequestExit();

	// Register/Get thread names for Android specific threads
	static CORE_API void RegisterThreadName(const char* Name, uint32 ThreadId);
	static CORE_API const char* GetThreadName(uint32 ThreadId);

	static CORE_API void ShowConsoleWindow();

	static CORE_API FDelegateHandle AddNetworkListener(FOnNetworkConnectionChangedDelegate&& InNewDelegate);
	static CORE_API bool RemoveNetworkListener(FDelegateHandle Handle);

	static CORE_API float GetVirtualKeyboardInputHeight();

#if !UE_BUILD_SHIPPING
	static CORE_API bool IsConsoleOpen();
#endif

private:
	static CORE_API const ANSICHAR* CodeToString(int Signal, int si_code);
	static CORE_API EDeviceScreenOrientation DeviceOrientation;

#if USE_ANDROID_JNI
	enum class EAndroidScreenOrientation
	{
		SCREEN_ORIENTATION_UNSPECIFIED = -1,
		SCREEN_ORIENTATION_LANDSCAPE = 0,
		SCREEN_ORIENTATION_PORTRAIT = 1,
		SCREEN_ORIENTATION_USER = 2,
		SCREEN_ORIENTATION_BEHIND = 3,
		SCREEN_ORIENTATION_SENSOR = 4,
		SCREEN_ORIENTATION_NOSENSOR = 5,
		SCREEN_ORIENTATION_SENSOR_LANDSCAPE = 6,
		SCREEN_ORIENTATION_SENSOR_PORTRAIT = 7,
		SCREEN_ORIENTATION_REVERSE_LANDSCAPE = 8,
		SCREEN_ORIENTATION_REVERSE_PORTRAIT = 9,
		SCREEN_ORIENTATION_FULL_SENSOR = 10,
		SCREEN_ORIENTATION_USER_LANDSCAPE = 11,
		SCREEN_ORIENTATION_USER_PORTRAIT = 12,
	};
	
	static CORE_API int32 GetAndroidScreenOrientation(EDeviceScreenOrientation ScreenOrientation);
#endif // USE_ANDROID_JNI
};

typedef FAndroidMisc FPlatformMisc;
