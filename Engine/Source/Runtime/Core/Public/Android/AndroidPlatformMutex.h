// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformMutex.h"
#include "HAL/PThreadsRecursiveMutex.h"
#include "HAL/PThreadsSharedMutex.h"

namespace UE
{

using FPlatformRecursiveMutex = FPThreadsRecursiveMutex;
using FPlatformSharedMutex = FPThreadsSharedMutex;
using FPlatformSystemWideMutex = FPlatformSystemWideMutexNotImplemented;

} // UE
