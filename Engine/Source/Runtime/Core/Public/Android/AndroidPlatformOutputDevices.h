// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	AndroidOutputDevices.h: Android platform OutputDevices functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformOutputDevices.h"

struct FAndroidOutputDevices : public FGenericPlatformOutputDevices
{
};

typedef FAndroidOutputDevices FPlatformOutputDevices;
