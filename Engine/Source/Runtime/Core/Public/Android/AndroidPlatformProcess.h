// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	AndroidProcess.h: Android platform Process functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformProcess.h"

/** Dummy process handle for platforms that use generic implementation. */
struct FProcHandle : public TProcHandle<void*, nullptr>
{
public:
	/** Default constructor. */
	UE_FORCEINLINE_HINT constexpr FProcHandle()
		: TProcHandle()
	{}

	/** Initialization constructor. */
	UE_FORCEINLINE_HINT explicit FProcHandle( HandleType Other )
		: TProcHandle( Other )
	{}
};

/**
 * Android implementation of the Process OS functions
 **/
struct FAndroidPlatformProcess : public FGenericPlatformProcess
{
	static CORE_API void* GetDllHandle(const TCHAR* Filename);
	static CORE_API void FreeDllHandle(void* DllHandle);
	static CORE_API void* GetDllExport(void* DllHandle, const TCHAR* ProcName);
	static CORE_API const TCHAR* ComputerName();
	static CORE_API void SetThreadAffinityMask( uint64 AffinityMask );
	static CORE_API uint32 GetCurrentProcessId();
	static CORE_API uint32 GetCurrentCoreNumber();
	static CORE_API const TCHAR* BaseDir();
	static CORE_API const TCHAR* ExecutableName(bool bRemoveExtension = true);
	static CORE_API class FRunnableThread* CreateRunnableThread();
	static CORE_API bool CanLaunchURL(const TCHAR* URL);
	static CORE_API void LaunchURL(const TCHAR* URL, const TCHAR* Parms, FString* Error);
	static CORE_API FString GetGameBundleId();
	static CORE_API void SetThreadName(const TCHAR* ThreadName);
	static CORE_API bool GetPerFrameProcessorUsage(uint32 ProcessId, float& ProcessUsageFraction, float& IdleUsageFraction);
};

typedef FAndroidPlatformProcess FPlatformProcess;
