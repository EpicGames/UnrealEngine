// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	AndroidProperties.h - Basic static properties of a platform 
	These are shared between:
		the runtime platform - via FPlatformProperties
		the target platforms - via ITargetPlatform
==================================================================================*/

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformProperties.h"

#define ANDROID_DEFAULT_DEVICE_PROFILE_NAME TEXT("Android_Default")

/**
 * Implements Android platform properties.
 */
struct FAndroidPlatformProperties
	: public FGenericPlatformProperties
{
	static constexpr UE_FORCEINLINE_HINT bool HasEditorOnlyData()
	{
		return false;
	}

	static constexpr UE_FORCEINLINE_HINT const char* PlatformName()
	{
		return "Android";
	}

	static constexpr UE_FORCEINLINE_HINT const char* IniPlatformName()
	{
		return "Android";
	}

	static constexpr UE_FORCEINLINE_HINT const TCHAR* GetRuntimeSettingsClassName()
	{
		return TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings");
	}

	static constexpr UE_FORCEINLINE_HINT bool IsGameOnly()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool RequiresCookedData()
	{
		return true;
	}

	static UE_FORCEINLINE_HINT bool SupportsBuildTarget(EBuildTargetType TargetType)
	{
		return (TargetType == EBuildTargetType::Game);
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsAutoSDK()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsHighQualityLightmaps()
	{
		return true; // always true because of Vulkan
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsLowQualityLightmaps()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsDistanceFieldShadows()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsTextureStreaming()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMinimize()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsQuit()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool HasFixedResolution()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool AllowsFramerateSmoothing()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool AllowsCallStackDumpDuringAssert()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsAudioStreaming()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMeshLODStreaming()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMemoryMappedFiles()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMemoryMappedAudio()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMemoryMappedAnimation()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT int64 GetMemoryMappingAlignment()
	{
		// Cook for largest page size available.
		// Data cooked for 16kb page size will work for 4kb page size firmware.
		// TODO: THIS IS WRONG!
		return 16384;
	}
};

#ifdef PROPERTY_HEADER_SHOULD_DEFINE_TYPE
typedef FAndroidPlatformProperties FPlatformProperties;
#endif
