// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	AndroidPlatformRunnableThread.h: Android platform threading functions
==============================================================================================*/

#pragma once

#include "Runtime/Core/Private/HAL/PThreadRunnableThread.h"
#include "Android/AndroidPlatformMisc.h"
#include "AndroidPlatform.h"

/**
* Android implementation of the pthread functions
**/
class FRunnableThreadAndroid : public FRunnableThreadPThread
{
	enum
	{
		AndroidThreadNameLimit = 15			// android name limit is 16, but we'll go with 15 to be safe and match the linux implementation
	};

public:
    FRunnableThreadAndroid() : FRunnableThreadPThread()
    {
    }

	~FRunnableThreadAndroid()
	{
		// Call the parent destructor body before the parent does it - see comment on that function for explanation why.
		FRunnableThreadPThread_DestructorBody();
	}

private:

	/**
	 * Allows a platform subclass to setup anything needed on the thread before running the Run function
	 */
	virtual void PreRun()
	{
		FString SizeLimitedThreadName = ThreadName;

		if (SizeLimitedThreadName.Len() > AndroidThreadNameLimit)
		{
			// first, attempt to cut out common and meaningless substrings
			SizeLimitedThreadName = SizeLimitedThreadName.Replace(TEXT("Thread"), TEXT(""));
			SizeLimitedThreadName = SizeLimitedThreadName.Replace(TEXT("Runnable"), TEXT(""));

			// if still larger
			if (SizeLimitedThreadName.Len() > AndroidThreadNameLimit)
			{
				FString Temp = SizeLimitedThreadName;

				// cut out the middle and replace with a substitute
				const TCHAR Dash[] = TEXT("-");
				const int32 DashLen = UE_ARRAY_COUNT(Dash) - 1;
				int NumToLeave = (AndroidThreadNameLimit - DashLen) / 2;

				SizeLimitedThreadName = Temp.Left(AndroidThreadNameLimit - (NumToLeave + DashLen));
				SizeLimitedThreadName += Dash;
				SizeLimitedThreadName += Temp.Right(NumToLeave);

				check(SizeLimitedThreadName.Len() <= AndroidThreadNameLimit);
			}
		}

		FPlatformProcess::SetThreadName(*SizeLimitedThreadName);
	}

	/**
	 * Allows platforms to adjust stack size
	 */
	virtual uint32 AdjustStackSize(uint32 InStackSize)
	{
		InStackSize = FRunnableThreadPThread::AdjustStackSize(InStackSize);

		// If it's set, make sure it's at least 128 KB or stack allocations (e.g. in Logf) may fail
		if (InStackSize && InStackSize < 128 * 1024)
		{
			InStackSize = 128 * 1024;
		}

		return InStackSize;
	}

#if ANDROID_USE_NICE_VALUE_THREADPRIORITY
	int32 ErrorLogLimit = 5;
	virtual void SetThreadPriority(pthread_t InThread, EThreadPriority NewPriority) override;
#endif
};
