// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	AndroidPlatformStackWalk.h: Android platform stack walk functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformStackWalk.h"

struct FAsyncThreadBackTrace
{
	std::atomic<int32> Flag;		//0 or 1 to indicate if stack capture has been finished
	int32 Depth;
	static constexpr int StackTraceMaxDepth = 100;
	uint64 BackTrace[StackTraceMaxDepth];
	uint32 ThreadID;
	static constexpr int MaxThreadName = 20;
	char ThreadName[MaxThreadName];
};

/**
* Android platform stack walking
*/
struct FAndroidPlatformStackWalk : public FGenericPlatformStackWalk
{
	typedef FGenericPlatformStackWalk Parent;

	static CORE_API void ProgramCounterToSymbolInfo(uint64 ProgramCounter, FProgramCounterSymbolInfo& out_SymbolInfo);
	static CORE_API uint32 CaptureStackBackTrace(uint64* BackTrace, uint32 MaxDepth, void* Context = nullptr);
	// Fast stack backtrace, only for tracing.
	// Will NOT work from signal handlers, is NOT suitable for crashreporting. Will not see inline functions.
	static CORE_API uint32 CaptureStackBackTraceViaFramePointerWalking(uint64* BackTrace, uint32 MaxDepth);
	static CORE_API bool SymbolInfoToHumanReadableString(const FProgramCounterSymbolInfo& SymbolInfo, ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize);

	static CORE_API uint32 CaptureThreadStackBackTrace(uint64 ThreadId, uint64* BackTrace, uint32 MaxDepth, void* Context = nullptr);
	static CORE_API int CaptureThreadStackBackTraceAsync(FAsyncThreadBackTrace* BackTrace);

	static CORE_API void HandleBackTraceSignal(siginfo* Info, void* Context);

	static CORE_API bool InitStackWalking();
};

typedef FAndroidPlatformStackWalk FPlatformStackWalk;
