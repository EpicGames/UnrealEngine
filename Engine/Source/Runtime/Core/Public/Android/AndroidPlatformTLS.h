// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	AndroidTLS.h: Android platform TLS (Thread local storage and thread ID) functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformTLS.h"
#include "Android/AndroidSystemIncludes.h"

/**
 * Android implementation of the TLS OS functions
 */
struct FAndroidTLS : public FGenericPlatformTLS
{
	/**
	 * Returns the currently executing thread's id
	 */
	static inline uint32 GetCurrentThreadId(void)
	{
		static_assert(sizeof(pid_t) <= sizeof(uint32), "pid_t is larger than uint32, reconsider implementation of GetCurrentThreadId()");
		return static_cast<uint32>(gettid());
//		return pthread_self();
	}

	/**
	 * Allocates a thread local store slot
	 */
	static inline uint32 AllocTlsSlot(void)
	{
		// allocate a per-thread mem slot
		pthread_key_t Key = 0;
		if (pthread_key_create(&Key, NULL) != 0)
		{
			Key = InvalidTlsSlot;  // matches the Windows TlsAlloc() retval //@todo android: should probably check for this below, or assert out instead
		}
		return Key;
	}

	/**
	 * Sets a value in the specified TLS slot
	 *
	 * @param SlotIndex the TLS index to store it in
	 * @param Value the value to store in the slot
	 */
	static UE_FORCEINLINE_HINT void SetTlsValue(uint32 SlotIndex,void* Value)
	{
		pthread_setspecific((pthread_key_t)SlotIndex, Value);
	}

	/**
	 * Reads the value stored at the specified TLS slot
	 *
	 * @return the value stored in the slot
	 */
	static UE_FORCEINLINE_HINT void* GetTlsValue(uint32 SlotIndex)
	{
		return pthread_getspecific((pthread_key_t)SlotIndex);
	}

	/**
	 * Frees a previously allocated TLS slot
	 *
	 * @param SlotIndex the TLS index to store it in
	 */
	static UE_FORCEINLINE_HINT void FreeTlsSlot(uint32 SlotIndex)
	{
		pthread_key_delete((pthread_key_t)SlotIndex);
	}
};

typedef FAndroidTLS FPlatformTLS;
