// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FAndroidPlatformThermal
{
	enum EForecastPeriod { INSTANT, ONE_SEC, FIVE_SEC, TEN_SEC, NUM_FORECASTPERIODS };
	// returns the 'thermal stress' value for the forecasted period.
	// This value is a hint for the extent of thermal throttling the device will experience during the time scale provided.
	// The implementation is provided by PowerManager.getThermalHeadroom(). 
	// Values of 1.0 or above indicate the current workload can not be sustained and thermal throttling will occur.
	static CORE_API float GetThermalStress(EForecastPeriod ForecastPeriod);
};

