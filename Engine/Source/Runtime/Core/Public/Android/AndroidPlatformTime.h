// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	AndroidTime.h: Android platform Time functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformTime.h"
#include <sys/time.h>

//@todo android: this entire file

/**
 * Android implementation of the Time OS functions
 */
struct FAndroidTime : public FGenericPlatformTime
{
	// android uses BSD time code from GenericPlatformTime
	static inline double Seconds()
	{
		struct timespec ts;
		clock_gettime(CLOCK_MONOTONIC, &ts);
		return ((double) ts.tv_sec) + (((double) ts.tv_nsec) / 1000000000.0);
	}

	static inline uint32 Cycles()
	{
		struct timespec ts;
		clock_gettime(CLOCK_MONOTONIC, &ts);
		return (uint32) ((((uint64)ts.tv_sec) * 1000000ULL) + (((uint64)ts.tv_nsec) / 1000ULL));
	}

	static inline uint64 Cycles64()
	{
		struct timespec ts;
		clock_gettime(CLOCK_MONOTONIC, &ts);
		return ((((uint64)ts.tv_sec) * 1000000ULL) + (((uint64)ts.tv_nsec) / 1000ULL));
	}
};

typedef FAndroidTime FPlatformTime;
