// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "CoreMinimal.h"

/*
* Invokes android's built-in perfetto capture API.
*
* Usage:
* FString ProfileName = FAndroidProfiler::StartCapture(ArgsString, [](const FAndroidProfiler::FProfileResults& Results)
				{ 
					// lambda to be called when capture completes.
					// This can be called from a random thread.
					// params: 
					// Results.ProfileName	== the tag/name used when initiating the profiling session.
					// Results.Error		== Any errors or warnings generated during the session
					// Results.FilePath		== empty on error. otherwise contains the path to the perfetto capture.
					//							the file is in the apps internal files folder. it is up to the application
					//							to move or delete this file.
				} );
* 
* where	ArgsString = "callstack [optional args]"
*		ArgsString = "heap		[optional args]"
*		ArgsString = "system	[optional args]"
*
* optional args can be any of the following (space delimited):
* 
*  duration=x								(in seconds, default 10)
*  buffersize=x								(in kb, default 1000)
*  profilename=x							(unique name for the profile session, default 'profile')
*											(you cannot have concurrent sessions with the same name.) 
*  samplingfreq=x							(callstack profile only. number of samples per second, default 100)
*  samplingintervalbytes=x					(heap profile only. alloc granularity of heap samples, default 100)
*  bufferfillpolicy=[ringbuffer|discard]	(system profile only. default ringbuffer)
*
* example: callstack samplingfreq=1000 duration=30
*
* to stop an in progress capture:
*	param ProfileName is the name/tag of the profile to cancel (i.e. as returned from StartCapture)
* FAndroidProfiler::StopCapture(ProfileName);
*
*/

class FAndroidProfiler
{
public:

	struct FProfileResults
	{
		FString ProfileName;// string used to identify the profile session.
		FString Error;		// Any errors or warnings produced
		FString FilePath;	// path to any output produced. empty on error.
	};

	// Returns an FString used to identify the profile session.
	static FString StartCapture(const FString& Args, TUniqueFunction<void(const FAndroidProfiler::FProfileResults& Results)> OnFinish);

	// stop an in-progress profile session.
	static void StopCapture(const FString& ProfileName);

private:
	static inline FCriticalSection ProfilerCS;

	// Called when a profile session ends. (either on success or on error)
	static void OnProfileFinish(const FAndroidProfiler::FProfileResults& Results);

	struct FActiveSessions
	{
		TUniqueFunction<void(const FProfileResults& Results)> OnFinish;
	};
	static inline TMap<FString, FActiveSessions> ActiveSessions;

	friend struct FAndroidProfilerInternal;
};