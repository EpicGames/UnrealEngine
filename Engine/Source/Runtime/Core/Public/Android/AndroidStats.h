// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

enum class EDeviceThermalState : int32;

class FAndroidStats
{
public:
	static void Init();
	static void UpdateAndroidStats();
	static void OnThermalStatusChanged(int Status);
	static void OnTrimMemory(int TrimLevel);
	static void LogGPUStats();

	// Stat accessors
	static EDeviceThermalState GetThermalStatus();
};
