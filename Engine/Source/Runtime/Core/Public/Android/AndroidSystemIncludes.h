// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <stdio.h>
#include <stdlib.h>
#include <stdint.h>
#include <ctype.h>
#include <wctype.h>
#include <pthread.h>
#include <limits.h>
#include <fcntl.h>
#include <sys/stat.h>
#include <utime.h>
#include <unistd.h>

// Set up compiler pragmas, etc
#include "Android/AndroidPlatformCompilerSetup.h"
#include "CoreTypes.h"

//@todo android: verify malloc alignment or change?
#define _aligned_malloc(Size,Align) malloc(Size)
#define _aligned_realloc(Ptr,Size,Align) realloc(Ptr,Size)
#define _aligned_free(Ptr) free(Ptr)

int vswprintf(TCHAR *dst, int count, const TCHAR *fmt, va_list arg);

/*
// Define TEXT early	//@todo android:
#define TEXT(x) L##x

// map the Windows functions (that Unreal unfortunately uses by default) to normal functions
#define _alloca alloca
*/

struct tagRECT
{
	int32 left;
	int32 top;
	int32 right;
	int32 bottom;
};
typedef struct tagRECT RECT;
