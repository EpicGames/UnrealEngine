// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#import <Foundation/Foundation.h>


@interface FAppleAsyncTask : NSObject
{
@private
	/** Whether or not the task is ready to have GameThread callback called (set on iOS thread) */
	int32 bIsReadyForGameThread;
}

/** Extra data for this async task */
@property (retain) id UserData;

/**
 * Code to run on the game thread when the async task completes 
 * @return true when the task is complete, and it will be pulled from the list of tasks and destroyed
 */
@property (copy) bool (^GameThreadCallback)(void);


/**
 * Create an async task object, set the block, and mark it as ready to be processed on main thread
 * For advanced uses (ie, setting UserData or delaying when it's ready for game thread, use the usual [[alloc] init] pattern
 *
 * Note, this doesn't return an AsyncObject so no one is tempted to use it - once it's ready for game thread, it's dangerous to touch it
 * This is callable on any thread, not just main thread
 */
+ (void)CreateTaskWithBlock:(bool (^)(void))Block;

/**
 * Mark that the task is complete on the iOS thread, and now the
 * GameThread can be fired (the Task is unsafe to use after this call)
 */
- (void)FinishedTask;

/**
 * Tick all currently running tasks
 */
+ (void)ProcessAsyncTasks;

@end
