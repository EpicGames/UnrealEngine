// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if !PLATFORM_TVOS && WITH_APPLE_CONTACTS

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#include <Contacts/Contacts.h>

/**
* Apple implementation of accessing device Contacts.
* @warning You will not be able to use this class (access will be silently denied, or worse, your app may crash), unless you have a valid NSContactsUsageDescription string in your plist.
**/
class CORE_API FAppleContactManager 
{
public:
	enum class ELabeledContactType
	{
		Home,
		Work,
		Mobile,
		Other
	};
	
	struct FLabeledContactValue
	{
		ELabeledContactType Type;
		FString Value;
	};
	
	struct FAppleContact 
	{
		FString FamilyName;
		FString GivenName;
		FString NickName;
		TArray<FLabeledContactValue> PhoneNumbers;
		TArray<FLabeledContactValue> EmailAddresses;
	};
	
	FAppleContactManager();
	
	bool HaveContactAccessAuthorization();
	
	void RequestContactAccess();
	
	TArray<FAppleContact> GetAllContacts();
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnContactsPermissionChanged, bool);
	FOnContactsPermissionChanged OnContactsPermissionChanged;
private:
	ELabeledContactType GetContactTypeForString(NSString* Label, bool bPhoneNumber);
	
	mutable __strong CNContactStore* ContactStore;
};

#endif
