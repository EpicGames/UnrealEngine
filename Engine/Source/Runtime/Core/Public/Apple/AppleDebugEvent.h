// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

struct FAppleDebugEvent
{
	void const* Tag;
	uint32 Color;
	uint64 Code;
	dispatch_block_t Destructor;
};