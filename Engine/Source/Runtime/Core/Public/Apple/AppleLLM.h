// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/LowLevelMemTracker.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER

#define LLM_SCOPE_APPLE(Tag) LLM_SCOPE((ELLMTag)Tag)
#define LLM_PLATFORM_SCOPE_APPLE(Tag) LLM_PLATFORM_SCOPE((ELLMTag)Tag)

#define LLM_TAG_APPLE_NUM_METAL_TAGS_RESERVED 5

enum class ELLMTagApple : LLM_TAG_TYPE
{
	ObjectiveC = (LLM_TAG_TYPE)ELLMTag::PlatformTagStart, // Use Instruments for detailed breakdown!

	/* Reserved for Metal tags to be utlized in MetalLLM.h */
	AppleMetalTagsStart,
	AppleMetalTagsEnd = (LLM_TAG_TYPE)((LLM_TAG_TYPE)AppleMetalTagsStart + LLM_TAG_APPLE_NUM_METAL_TAGS_RESERVED),
	
	Count
};
static_assert((int32)ELLMTagApple::Count <= (int32)ELLMTag::PlatformTagEnd, "too many ELLMTagApple tags");

namespace AppleLLM
{
	void Initialise();
}

#else

#define LLM_SCOPE_APPLE(...)
#define LLM_PLATFORM_SCOPE_APPLE(...)

#endif		// #if ENABLE_LOW_LEVEL_MEM_TRACKER

