// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <libkern/OSAtomic.h>
#include <objc/objc.h>
#include <Foundation/NSObject.h>

#include <mach/mach.h>

NS_ASSUME_NONNULL_BEGIN


// Support for passing LLM alloc data to XCode Instruments.
// Use USE_APPLE_SUPPORT_INSTRUMENTED_ALLOCS to enable it.
// As APPLE_SUPPORT_INSTRUMENTED_ALLOCS define only enables the LLM API, because changes to this header trigger wide recompilation.
#ifndef APPLE_SUPPORT_INSTRUMENTED_ALLOCS
	#define APPLE_SUPPORT_INSTRUMENTED_ALLOCS (!UE_BUILD_SHIPPING)
#endif // APPLE_SUPPORT_INSTRUMENTED_ALLOCS

/**
 * NSObject subclass that can be used to override the allocation functions to go through UE4's memory allocator.
 * This ensures that memory allocated by custom Objective-C types can be tracked by UE4's tools and 
 * that we benefit from the memory allocator's efficiencies.
 */
OBJC_EXPORT @interface FApplePlatformObject : NSObject
{
	@private
		OSQueueHead* AllocatorPtr;
}

/** Sub-classes should override to provide the OSQueueHead* necessary to allocate from - handled by the macro */
+ (nullable OSQueueHead*)classAllocator;

/** Sub-classes should override allocWithZone & alloc to call allocClass */
+ (id)allocClass: (Class)NewClass;

/** Override the core NSObject deallocation function to correctly destruct */
- (void)dealloc;

@end

#define APPLE_PLATFORM_OBJECT_ALLOC_OVERRIDES(ClassName)		\
+ (nullable OSQueueHead*)classAllocator							\
{																\
static OSQueueHead Queue = OS_ATOMIC_QUEUE_INIT;			\
return &Queue;												\
}																\
+ (id)allocWithZone:(NSZone*) Zone								\
{																\
return (ClassName*)[FApplePlatformObject allocClass:self];	\
}																\
+ (id)alloc														\
{																\
return (ClassName*)[FApplePlatformObject allocClass:self];	\
}

NS_ASSUME_NONNULL_END