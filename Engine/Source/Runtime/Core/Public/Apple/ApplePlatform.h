// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	ApplePlatform.h: Common setup for Apple platforms
==============================================================================================*/

#pragma once

#include "Clang/ClangPlatform.h"

// Technically the underlying platform has 128bit atomics, but clang might not issue optimal code
#define PLATFORM_HAS_128BIT_ATOMICS						0

// Base defines, defaults are commented out
#define PLATFORM_LITTLE_ENDIAN							1
#define PLATFORM_SEH_EXCEPTIONS_DISABLED				1
#define PLATFORM_SUPPORTS_PRAGMA_PACK					1
#define PLATFORM_ENABLE_VECTORINTRINSICS				1
#define PLATFORM_USE_SYSTEM_VSWPRINTF					0
#define PLATFORM_COMPILER_DISTINGUISHES_INT_AND_LONG	1
#define PLATFORM_WCHAR_IS_4_BYTES						1
#define PLATFORM_HAS_BSD_TIME							1
#define PLATFORM_HAS_BSD_IPV6_SOCKETS					1
#define PLATFORM_HAS_BSD_SOCKET_FEATURE_MSG_DONTWAIT	1
#define PLATFORM_HAS_MULTITHREADED_PREMAIN				1
#define PLATFORM_SUPPORTS_TEXTURE_STREAMING				1
#define PLATFORM_SUPPORTS_STACK_SYMBOLS					1
#define PLATFORM_IS_ANSI_MALLOC_THREADSAFE				1
#define PLATFORM_SUPPORTS_MIMALLOC						1

#define PLATFORM_BREAK()                                __builtin_debugtrap()

#define PLATFORM_CODE_SECTION(Name)						__attribute__((section("__TEXT,__" Name ",regular,pure_instructions"))) \
														__attribute__((aligned(4)))

// Ensure we can use this builtin - seems to be present on Clang 9, GCC 11 and MSVC 19.26,
// but gives spurious "non-void function 'BitCast' should return a value" errors on some
// Mac and Android toolchains when building PCHs, so avoid those.
#undef PLATFORM_COMPILER_SUPPORTS_BUILTIN_BITCAST
#define PLATFORM_COMPILER_SUPPORTS_BUILTIN_BITCAST (__clang_major__ >= 13)

// Function type macros.
#define VARARGS																/* Functions with variable arguments */
#define CDECL																/* Standard C function */
#define STDCALL																/* Standard calling convention */
#define FORCENOINLINE				__attribute__((noinline))				/* Force code to NOT be inline */
#define FUNCTION_CHECK_RETURN_END	__attribute__ ((warn_unused_result))	/* Warn that callers should not ignore the return value. */
#define FUNCTION_NO_RETURN_END		__attribute__ ((noreturn))				/* Indicate that the function never returns. */

#define ABSTRACT abstract

// We can use pragma optimisations on and off as of Apple LLVM 7.3.0 but not before.
#if (__clang_major__ > 7) || (__clang_major__ == 7 && __clang_minor__ >= 3)
#	define PRAGMA_DISABLE_OPTIMIZATION_ACTUAL _Pragma("clang optimize off")
#	define PRAGMA_ENABLE_OPTIMIZATION_ACTUAL  _Pragma("clang optimize on")
#endif

// Alignment.
#define GCC_PACK(n)		__attribute__((packed,aligned(n)))
#define GCC_ALIGN(n)	__attribute__((aligned(n)))

// operator new/delete operators
// As of 10.9 we need to use _NOEXCEPT & cxx_noexcept compatible definitions
#if __has_feature(cxx_noexcept)
#	define OPERATOR_NEW_THROW_SPEC
#else
#	define OPERATOR_NEW_THROW_SPEC		throw (std::bad_alloc)
#endif
#define OPERATOR_DELETE_THROW_SPEC		noexcept
#define OPERATOR_NEW_NOTHROW_SPEC		noexcept
#define OPERATOR_DELETE_NOTHROW_SPEC	noexcept

// DLL export and import definitions
#define DLLEXPORT			__attribute__((visibility("default")))
#define DLLIMPORT			__attribute__((visibility("default")))