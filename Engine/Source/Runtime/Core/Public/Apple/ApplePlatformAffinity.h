// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformAffinity.h"

class CORE_API FApplePlatformAffinity : public FGenericPlatformAffinity
{
public:
	static EThreadPriority GetRenderingThreadPriority()
	{
		return TPri_TimeCritical;
	}
	
	static EThreadPriority GetRHIThreadPriority()
	{
		return TPri_Highest;
	}
};
