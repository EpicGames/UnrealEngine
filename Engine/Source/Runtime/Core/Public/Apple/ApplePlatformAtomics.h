// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	ApplePlatformAtomics.h: Apple platform Atomics functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformAtomics.h"
#include "CoreTypes.h"
#include "Clang/ClangPlatformAtomics.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

/**
 * Apple implementation of the Atomics OS functions
 **/
struct CORE_API FApplePlatformAtomics : public FClangPlatformAtomics
{
#if	PLATFORM_HAS_128BIT_ATOMICS
	static FORCEINLINE bool InterlockedCompareExchange128( volatile FInt128* Dest, const FInt128& Exchange, FInt128* Comparand )
	{
		__uint128_t* Expected = ((__uint128_t*)Comparand);
		__uint128_t Desired = ((__uint128_t const&)Exchange);
		return __atomic_compare_exchange_16((volatile __uint128_t*)Dest, Expected, Desired, false, __ATOMIC_SEQ_CST, __ATOMIC_SEQ_CST);
	}
	
	/**
	 * @return true, if the processor we are running on can execute compare and exchange 128-bit operation.
	 * @see cmpxchg16b, early AMD64 processors don't support this operation.
	 */
	static FORCEINLINE bool CanUseCompareExchange128()
	{
		return true;
	}
#endif
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

typedef FApplePlatformAtomics FPlatformAtomics;

