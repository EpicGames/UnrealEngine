// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Make certain warnings always be warnings, even despite -Werror.
// Rationale: we don't want to suppress those as there are plans to address them (e.g. UE-12341), but breaking builds due to these warnings is very expensive
// since they cannot be caught by all compilers that we support. They are deemed to be relatively safe to be ignored, at least until all SDKs/toolchains start supporting them.
#ifdef __BUILDING_WITH_FASTBUILD__
	#pragma clang diagnostic ignored "-Wparentheses-equality"
#else
	#pragma clang diagnostic warning "-Wparentheses-equality"
#endif
#pragma clang diagnostic ignored "-Winconsistent-missing-override"
#pragma clang diagnostic ignored "-Wundefined-bool-conversion"
#pragma clang diagnostic ignored "-Wunused-local-typedef"
#pragma clang diagnostic ignored "-Wconstant-logical-operand"
#pragma clang diagnostic ignored "-Wreserved-user-defined-literal"
#pragma clang diagnostic ignored "-Wdelete-non-virtual-dtor"
#pragma clang diagnostic ignored "-Wlogical-op-parentheses"
#pragma clang diagnostic ignored "-Wunused-variable"
#pragma clang diagnostic ignored "-Wunused-value"
#pragma clang diagnostic ignored "-Wunused-function" // This will hide the warnings about static functions in headers that aren't used in every single .cpp file
#pragma clang diagnostic ignored "-Wswitch" // This hides the "enumeration value 'XXXXX' not handled in switch [-Wswitch]" warnings - we should maybe remove this at some point and add UE_LOG(, Fatal, ) to default cases
#pragma clang diagnostic ignored "-Wtautological-compare" // This hides the "warning : comparison of unsigned expression < 0 is always false" type warnings due to constant comparisons, which are possible with template arguments
#pragma clang diagnostic ignored "-Wunused-private-field" // This will prevent the issue of warnings for unused private variables.
#pragma clang diagnostic ignored "-Winvalid-offsetof" // needed to suppress warnings about using offsetof on non-POD types.
#pragma clang diagnostic ignored "-Wgnu-string-literal-operator-template" // we use this feature to allow static FNames.
#pragma clang diagnostic ignored "-Wdeprecated-register" // Needed for Alembic third party lib
#pragma clang diagnostic ignored "-Winconsistent-missing-override" // too many missing overrides...
#pragma clang diagnostic ignored "-Wunused-local-typedef" // PhysX has some, hard to remove
#pragma clang diagnostic ignored "-Wlogical-op-parentheses"
#pragma clang diagnostic ignored "-Wunknown-pragmas"
#pragma clang diagnostic ignored "-Wundefined-var-template"

#if __has_warning("-Wunused-but-set-variable")
#pragma clang diagnostic ignored "-Wunused-but-set-variable"
#endif

// Apple LLVM 9.1.0 (Xcode 9.3)
#if (__clang_major__ > 9) || (__clang_major__ == 9 && __clang_minor__ >= 1)
#pragma clang diagnostic ignored "-Wunused-lambda-capture"
#endif

// Apple LLVM 9.0 (Xcode 9.0)
#if (__clang_major__ >= 9)
#pragma clang diagnostic ignored "-Wunguarded-availability-new"
#endif

// Apple LLVM 8.1.0 (Xcode 8.3) introduced -Wundefined-var-template
#if (__clang_major__ > 8) || (__clang_major__ == 8 && __clang_minor__ >= 1)
#pragma clang diagnostic ignored "-Wundefined-var-template"
#pragma clang diagnostic ignored "-Wnullability-inferred-on-nested-type"
#pragma clang diagnostic ignored "-Wobjc-protocol-property-synthesis"
#pragma clang diagnostic ignored "-Wnullability-completeness-on-arrays"
#pragma clang diagnostic ignored "-Wnull-dereference"
#pragma clang diagnostic ignored "-Wnullability-completeness" // We are not interoperable with Swift so we DON'T care about nullability qualifiers
#pragma clang diagnostic ignored "-Wnonportable-include-path" // Ideally this one would be set in MacToolChain, but we don't have a way to check the compiler version in there yet
#endif

#if (__clang_major__ > 8)
#pragma clang diagnostic ignored "-Wunguarded-availability-new"
#endif

// We can use pragma optimisation's on and off as of Apple LLVM 7.3.0 but not before.
#if (__clang_major__ > 7) || (__clang_major__ == 7 && __clang_minor__ >= 3)
#define PRAGMA_DISABLE_OPTIMIZATION_ACTUAL _Pragma("clang optimize off")
#define PRAGMA_ENABLE_OPTIMIZATION_ACTUAL  _Pragma("clang optimize on")
#endif

#define PRAGMA_DEFAULT_VISIBILITY_START _Pragma("GCC visibility push(default)")
#define PRAGMA_DEFAULT_VISIBILITY_END   _Pragma("GCC visibility pop")

#include "Clang/ClangPlatformCompilerPreSetup.h"
