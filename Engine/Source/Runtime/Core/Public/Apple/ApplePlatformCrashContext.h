// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	ApplePlatformCrashContext.h: Apple platform crash context declaration
==============================================================================================*/

#pragma once

#include "GenericPlatform/GenericPlatformCrashContext.h"

/**
 * Declaration for common crash context implementation for Apple platforms.
 */
struct CORE_API FApplePlatformCrashContext : public FGenericCrashContext
{
	FApplePlatformCrashContext(ECrashContextType InType, const TCHAR* InErrorMessage);
	~FApplePlatformCrashContext();
	
	/** Signal number */
	int32 Signal;
	
	/** Additional signal info */
	siginfo_t* Info;
	
	/** Thread context */
	ucontext_t*	Context;
	
	/** The PC of where the error being reported occurred. This is used to figure
	 * out the number of frames to ignore */
	void* ErrorFrame = nullptr;
	
	/** Memory reserved for "exception" (signal) info */
	char SignalDescription[128];
	
	/** Memory reserved for minidump-style callstack info */
	char MinidumpCallstackInfo[16384];
	
	/** To be returned by GetCallstackProperty() */
	TCHAR CallstackProperty[16384];
	
	/**
	 * Inits the crash context from data provided by a signal handler.
	 * @param InSignal number (SIGSEGV, etc)
	 * @param InInfo additional info (e.g. address we tried to read, etc)
	 * @param InContext thread context
	 */
	void InitFromSignal(int32 InSignal, siginfo_t* InInfo, void* InContext);
	
	/** Initialise context for reporting crash. */
	int32 ReportCrash(void) const;
	
	/** Generates a string representation for the execption/signal Info. */
	static void CreateExceptionInfoString(int32 Signal, siginfo_t* Info);
	
	/** Converts the PLCrashReporter minidump */
	static void ConvertMinidump(char const* OutputPath, char const* InputPath);
	
	/** Write a line of UTF-8 to a file */
	static void WriteLine(int ReportFile, const ANSICHAR* Line = NULL);
	
	/** Serializes UTF string to UTF-16 */
	static void WriteUTF16String(int ReportFile, const TCHAR* UTFString4BytesChar, uint32 NumChars);
	
	/** Serializes UTF string to UTF-16 */
	static void WriteUTF16String(int ReportFile, const TCHAR* UTFString4BytesChar);
	
	/** Writes UTF-16 line to a file */
	static void WriteLine(int ReportFile, const TCHAR* Line);
	
	/** Async-safe ItoA */
	static ANSICHAR* ItoANSI(uint64 Val, uint64 Base, uint32 Len = 0);
	
	/** Async-safe ItoT */
	static TCHAR* ItoTCHAR(uint64 Val, uint64 Base, uint32 Len = 0);

protected:

	/** Allow platform implementations to provide a callstack property. Primarily used when non-native code triggers a crash. */
	CORE_API virtual const TCHAR* GetCallstackProperty() const;
};
