// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	FApplePlatformDebugEvents.h: Apple platform debug events
==============================================================================================*/

#pragma once
#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#if !defined(APPLE_PROFILING_ENABLED)
#define APPLE_PROFILING_ENABLED 0
#endif

#if APPLE_PROFILING_ENABLED

DECLARE_LOG_CATEGORY_EXTERN(LogInstruments, Display, All);

struct FAppleDebugEvent;

struct FApplePlatformDebugEvents
{
	static void DebugSignPost(uint16 Code, uintptr_t Arg1, uintptr_t Arg2, uintptr_t Arg3, uintptr_t Arg4);

	static void DebugSignPostStart(uint16 Code, uintptr_t Arg1, uintptr_t Arg2, uintptr_t Arg3, uintptr_t Arg4);

	static void DebugSignPostEnd(uint16 Code, uintptr_t Arg1, uintptr_t Arg2, uintptr_t Arg3, uintptr_t Arg4);
	
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
	struct FEvent
	{
		void const* Tag;
		uint32 Color;
		uint64 Code;
		dispatch_block_t Destructor;
	};
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
	
	static TArray<FAppleDebugEvent>* GetEventStack();
	
	static uint16 GetEventCode(FString String);
	
	static void BeginNamedEvent(const struct FColor& Color,const TCHAR* Text);
	
	static void BeginNamedEvent(const struct FColor& Color,const ANSICHAR* Text);
	
	static void EndNamedEvent();
	
	static uint32 TLSSlot;
};

#endif
