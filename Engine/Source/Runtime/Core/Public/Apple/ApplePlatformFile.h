// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	ApplePlatformFile.h: Apple platform File functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformFile.h"

template <typename FuncType> class TFunctionRef;

/**
 * Mac File I/O implementation
**/
class CORE_API FApplePlatformFile : public IPhysicalPlatformFile
{
protected:
	virtual FString NormalizeFilename(const TCHAR* Filename);
	virtual FString NormalizeDirectory(const TCHAR* Directory);
public:
	//~ For visibility of overloads we don't override
	using IPhysicalPlatformFile::IterateDirectory;
	using IPhysicalPlatformFile::IterateDirectoryStat;

	virtual bool FileExists(const TCHAR* Filename) override;
	virtual int64 FileSize(const TCHAR* Filename) override;
	virtual bool DeleteFile(const TCHAR* Filename) override;
	virtual bool IsReadOnly(const TCHAR* Filename) override;
	virtual bool MoveFile(const TCHAR* To, const TCHAR* From) override;
	virtual bool SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override;


	virtual FDateTime GetTimeStamp(const TCHAR* Filename) override;

	virtual void SetTimeStamp(const TCHAR* Filename, const FDateTime DateTime) override;

	virtual FDateTime GetAccessTimeStamp(const TCHAR* Filename) override;
	virtual FString GetFilenameOnDisk(const TCHAR* Filename) override;

	virtual ESymlinkResult IsSymlink(const TCHAR* Filename) override;

	virtual IFileHandle* OpenRead(const TCHAR* Filename, bool bAllowWrite = false) override;
	virtual IFileHandle* OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override;
	virtual FOpenMappedResult OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions = EOpenReadFlags::None, int64 MaximumSize = 0) override;
	virtual FOpenMappedResult OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions = EOpenMappedFlags::None, int64 MaximumSize = 0) override;
	virtual bool DirectoryExists(const TCHAR* Directory) override;
	virtual bool CreateDirectory(const TCHAR* Directory) override;
	virtual bool DeleteDirectory(const TCHAR* Directory) override;

	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override;

	virtual bool IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor) override;
	virtual bool IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor) override;

	virtual bool CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags = EPlatformFileRead::None, EPlatformFileWrite WriteFlags = EPlatformFileWrite::None) override;
protected:
	virtual FString ConvertToPlatformPath(const FString& Filename, bool bForWrite, bool bIsPublicWrite) { return Filename; }
	virtual int32 OpenFileForMemMap(const FString& NormalizedFilename, EOpenMappedFlags OpenOptions, FString& OutFinalPath);
	int GetModeFlagsForOpenWrite() const;
private:
	bool IterateDirectoryCommon(const TCHAR* Directory, const TFunctionRef<bool(struct dirent*)>& Visitor);

	int32 Stat(const TCHAR* Filename, struct stat* OutFileInfo);
};
