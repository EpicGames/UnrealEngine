// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	ApplePlatformMemory.h: Apple platform memory functions common across all Apple OSes
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformMemory.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include <libkern/OSAtomic.h>
#include <Foundation/NSObject.h>

#include <mach/mach.h>
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

// Support for passing LLM alloc data to XCode Instruments.
// Use USE_APPLE_SUPPORT_INSTRUMENTED_ALLOCS to enable it.
// As APPLE_SUPPORT_INSTRUMENTED_ALLOCS define only enables the LLM API, because changes to this header trigger wide recompilation.
#ifndef APPLE_SUPPORT_INSTRUMENTED_ALLOCS
	#define APPLE_SUPPORT_INSTRUMENTED_ALLOCS (!UE_BUILD_SHIPPING)
#endif // APPLE_SUPPORT_INSTRUMENTED_ALLOCS


/**
 *	Max implementation of the FGenericPlatformMemoryStats.
 */
struct FPlatformMemoryStats : public FGenericPlatformMemoryStats
{
	FPlatformMemoryStats()
		: FGenericPlatformMemoryStats()
		, MemoryPressureStatus(EMemoryPressureStatus::Unknown)
	{}
	
	EMemoryPressureStatus GetMemoryPressureStatus() const
	{
		if (MemoryPressureStatus == EMemoryPressureStatus::Unknown)
		{
			// if platform doesn't make use of MemoryPressureStatus, use default implementation
			return FGenericPlatformMemoryStats::GetMemoryPressureStatus();
		}
		else
		{
			return MemoryPressureStatus;
		}
	}
	
	EMemoryPressureStatus MemoryPressureStatus;
};

/**
 * Common Apple platform memory functions.
 */
struct CORE_API FApplePlatformMemory : public FGenericPlatformMemory
{
	//~ Begin FGenericPlatformMemory Interface
	static void Init();
	static FPlatformMemoryStats GetStats();
	static uint64 GetMemoryUsedFast();
	static const FPlatformMemoryConstants& GetConstants();
	static FMalloc* BaseAllocator();
	static bool PageProtect(void* const Ptr, const SIZE_T Size, const bool bCanRead, const bool bCanWrite);
	static void* BinnedAllocFromOS(SIZE_T Size);
	static void BinnedFreeToOS(void* Ptr, SIZE_T Size);
	static bool PtrIsOSMalloc( void* Ptr);
	static bool PtrIsFromNanoMalloc( void* Ptr);
	static bool IsNanoMallocAvailable();
	static void NanoMallocInit();
	UE_DEPRECATED(5.8, "This doesn't belong in a public interface")
	inline void SetAllocatorToUse() {}

	class FPlatformVirtualMemoryBlock : public FBasicVirtualMemoryBlock
	{
	public:

		FPlatformVirtualMemoryBlock()
		{
		}

		FPlatformVirtualMemoryBlock(void *InPtr, uint32 InVMSizeDivVirtualSizeAlignment)
			: FBasicVirtualMemoryBlock(InPtr, InVMSizeDivVirtualSizeAlignment)
		{
		}
		FPlatformVirtualMemoryBlock(const FPlatformVirtualMemoryBlock& Other) = default;
		FPlatformVirtualMemoryBlock& operator=(const FPlatformVirtualMemoryBlock& Other) = default;

		void Commit(size_t InOffset, size_t InSize);
		void Decommit(size_t InOffset, size_t InSize);
		void FreeVirtual();

		UE_FORCEINLINE_HINT void CommitByPtr(void *InPtr, size_t InSize)
		{
			Commit(size_t(((uint8*)InPtr) - ((uint8*)Ptr)), InSize);
		}

		UE_FORCEINLINE_HINT void DecommitByPtr(void *InPtr, size_t InSize)
		{
			Decommit(size_t(((uint8*)InPtr) - ((uint8*)Ptr)), InSize);
		}

		UE_FORCEINLINE_HINT void Commit()
		{
			Commit(0, GetActualSize());
		}

		UE_FORCEINLINE_HINT void Decommit()
		{
			Decommit(0, GetActualSize());
		}

		UE_FORCEINLINE_HINT size_t GetActualSize() const
		{
			return VMSizeDivVirtualSizeAlignment * GetVirtualSizeAlignment();
		}

		static FPlatformVirtualMemoryBlock AllocateVirtual(size_t Size, size_t InAlignment = FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment());
		static size_t GetCommitAlignment();
		static size_t GetVirtualSizeAlignment();
	};

    static bool GetLLMAllocFunctions(void* _Nonnull (* _Nonnull &OutAllocFunction)(size_t), void(* _Nonnull &OutFreeFunction)(void*, size_t), int32& OutAlignment);
#if APPLE_SUPPORT_INSTRUMENTED_ALLOCS
	static void OnLowLevelMemory_Alloc(void const* Pointer, uint64 Size, uint64 Tag);
	static void OnLowLevelMemory_Free(void const* Pointer, uint64 Size, uint64 Tag);
#endif // APPLE_SUPPORT_INSTRUMENTED_ALLOCS
    //~ End FGenericPlatformMemory Interface
	
	/** Setup the current default CFAllocator to use our malloc functions. */
	static void ConfigureDefaultCFAllocator(void);

	static bool CanOverallocateVirtualMemory();
	
#ifndef _MACH_VM_TYPES_H_
	typedef unsigned long vm_address_t;
#endif
	
	static vm_address_t NanoRegionStart;
	static vm_address_t NanoRegionEnd;
};
