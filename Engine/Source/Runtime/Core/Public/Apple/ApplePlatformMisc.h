// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	ApplePlatformMisc.h: Apple platform misc functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformMisc.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#else
#include "IOS/IOSSystemIncludes.h"
#endif

#include "Apple/ScopeAutoreleasePool.h"

#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "CoreTypes.h"

#ifndef APPLE_PROFILING_ENABLED
#define APPLE_PROFILING_ENABLED (UE_BUILD_DEBUG | UE_BUILD_DEVELOPMENT)
#endif

#ifndef WITH_IOS_SIMULATOR
#define WITH_IOS_SIMULATOR 0
#endif

#define UE_DEBUG_BREAK_IMPL() PLATFORM_BREAK()

template <typename FuncType, typename UserPolicy>
class TMulticastDelegateRegistration;

struct FDefaultTSDelegateUserPolicy;

/**
* Apple implementation of the misc OS functions
**/
struct CORE_API FApplePlatformMisc : public FGenericPlatformMisc
{
	static CORE_API void PlatformInit();
	static CORE_API void PlatformTearDown();

	static FString GetEnvironmentVariable(const TCHAR* VariableName);

#if !UE_BUILD_SHIPPING
	static bool IsDebuggerPresent();
#endif

	CORE_API static void MemoryBarrier();

	static void LocalPrint(const TCHAR* Message);
	static bool IsLocalPrintThreadSafe() { return true; }
	static const TCHAR* GetSystemErrorMessage(TCHAR* OutBuffer, int32 BufferCount, int32 Error);
	static uint32 GetLastError();
	static int32 NumberOfCores();
	
	/** @return Whether filehandles can be opened on one thread and read/written on another thread */
	static bool SupportsMultithreadedFileHandles()
	{
		// ApplePlatformFile currently uses thread-local lists that can close filehandles arbitrarily and reopen them at need, so filehandles are not transferrable between threads
		return false;
	}

	static void CreateGuid(struct FGuid& Result);
	static TArray<uint8> GetSystemFontBytes();
	static FString GetDefaultLocale();
	static FString GetDefaultLanguage();
	static FString GetLocalCurrencyCode();
	static FString GetLocalCurrencySymbol();

	static bool IsOSAtLeastVersion(const uint32 MacOSVersion[3], const uint32 IOSVersion[3], const uint32 TVOSVersion[3]);

#if APPLE_PROFILING_ENABLED
	/** Override to add Apple features when AppleProfiling is on, otherwise use version on FGenericPlatformMisc. */
	static void BeginNamedEvent(const struct FColor& Color, const TCHAR* Text);
	static void BeginNamedEvent(const struct FColor& Color, const ANSICHAR* Text);
	static void EndNamedEvent();
#endif

	//////// Platform specific
	static void* CreateAutoreleasePool();
	static void ReleaseAutoreleasePool(void *Pool);
	
	static CORE_API void SetEnvironmentVar(const TCHAR* InVariableName, const TCHAR* Value);
	
	static ENetworkConnectionType GetNetworkConnectionType();
	static bool IsDataSavingNetworkConnection();
	static bool HasActiveWiFiConnection();
	
	// Information about the properties of the network that a connection uses.
	struct FNetworkConnectionCharacteristics
	{
		// Indndicating whether the used network interface has a DNS server configured.
		bool bSupportsDNS   : 1 = false;
		// Indicating whether the used network interface can route IPv4 traffic.
		bool bSupportsIPv4  : 1 = false;
		// Indicating whether the used network interface can route IPv6 traffic.
		bool bSupportsIPv6  : 1 = false;
		// Indicating whether the used network interface is in Low Data Mode.
		// It is applicable to both WiFi & cellular connections.
		bool bIsConstrained : 1 = false;
		// Indicating whether the used network interface is considered
		// expensive, such as cellular or a personal hotspot.
		bool bIsExpensive   : 1 = false;
	};
	// Thread-safe event that is fired from a task-thread.
	static TMulticastDelegateRegistration<void(FNetworkConnectionCharacteristics), FDefaultTSDelegateUserPolicy>& OnNetworkConnectionCharacteristicsChanged();
	static FNetworkConnectionCharacteristics GetNetworkConnectionCharacteristics();
};
