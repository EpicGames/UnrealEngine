// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformProcess.h"

/**
 * Apple common  implementation of the Process OS functions
 **/
struct CORE_API FApplePlatformProcess : public FGenericPlatformProcess
{
	static const TCHAR* UserDir();
	static const TCHAR* UserTempDir();
	static const TCHAR* UserSettingsDir();
	static const TCHAR* ApplicationSettingsDir();
	static FString GetApplicationSettingsDir(const ApplicationSettingsContext& Settings);

	// Apple specific
	static const TCHAR* UserHomeDir();
};
