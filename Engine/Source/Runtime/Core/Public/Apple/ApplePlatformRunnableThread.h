// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	ApplePlatformRunnableThread.h: Apple platform threading functions
==============================================================================================*/

#pragma once

#include "Runtime/Core/Private/HAL/PThreadRunnableThread.h"

/**
* Apple implementation of the Process OS functions
**/
class FRunnableThreadApple : public FRunnableThreadPThread
{
public:
    FRunnableThreadApple()
    :   Pool(NULL)
    {
    }

	~FRunnableThreadApple()
	{
		// Call the parent destructor body before the parent does it - see comment on that function for explanation why.
		FRunnableThreadPThread_DestructorBody();
	}

private:
	/**
	 * Allows a platform subclass to setup anything needed on the thread before running the Run function
	 */
	virtual void PreRun() override
	{
		//@todo - zombie - This and the following build somehow. It makes no sense to me. -astarkey
		Pool = FPlatformMisc::CreateAutoreleasePool();
		pthread_setname_np(TCHAR_TO_ANSI(*ThreadName));
	}

	/**
	 * Allows a platform subclass to teardown anything needed on the thread after running the Run function
	 */
	virtual void PostRun() override
	{
		FPlatformMisc::ReleaseAutoreleasePool(Pool);
	}
	
	virtual int GetDefaultStackSize() override
	{
		// default is 512 KB, we need more
		return FPlatformMisc::GetDefaultStackSize();
	}
    
	/**
	 * Allows platforms to adjust stack size
	 */
	virtual uint32 AdjustStackSize(uint32 InStackSize) override
	{
		InStackSize = FRunnableThreadPThread::AdjustStackSize(InStackSize);
        
		// If it's set, make sure it's at least 128 KB or stack allocations (e.g. in Logf) may fail
		if (InStackSize < GetDefaultStackSize())
		{
			InStackSize = GetDefaultStackSize();
		}
        
		return InStackSize;
	}
	
	/**
	 * Converts an EThreadPriority to a value that can be used in pthread_setschedparam. Virtual
	 * so that platforms can override priority values
	 */
	virtual int32 TranslateThreadPriority(EThreadPriority Priority) override
	{
		// these are some default priorities
		switch (Priority)
		{
			// 0 is the lowest, 47 is the high priority for Apple
			case TPri_TimeCritical: return 47;
			case TPri_Highest:  return 45;
			case TPri_AboveNormal: return 37;
			case TPri_Normal: return 31;
			case TPri_SlightlyBelowNormal: return 30;
			case TPri_BelowNormal: return 25;
			case TPri_Lowest: return 20;
			case TPri_Num:
			default: UE_LOGF(LogHAL, Fatal, "Unknown Priority passed to FRunnableThreadApple::TranslateThreadPriority()"); return 31;
		}
	}
    
    void*      Pool;
};

