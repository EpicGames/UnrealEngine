// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	ApplePlatformStackWalk.h: Apple platform stack walk functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformStackWalk.h"

/**
 * Apple platform implementation of the misc OS functions
 */
struct CORE_API FApplePlatformStackWalk : public FGenericPlatformStackWalk
{
	static bool ProgramCounterToHumanReadableString( int32 CurrentCallDepth, uint64 ProgramCounter, ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, FGenericCrashContext* Context = nullptr);
	static void ProgramCounterToSymbolInfo( uint64 ProgramCounter, FProgramCounterSymbolInfo&  out_SymbolInfo);
	static uint32 CaptureStackBackTrace( uint64* BackTrace, uint32 MaxDepth, void* Context = nullptr);
	static uint32 CaptureThreadStackBackTrace(uint64 ThreadId, uint64* BackTrace, uint32 MaxDepth, void* Context = nullptr);
	static void ThreadStackWalkAndDump(ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, uint32 ThreadId);
	static int32 GetProcessModuleCount();
	static int32 GetProcessModuleSignatures(FStackWalkModuleInfo *ModuleSignatures, const int32 ModuleSignaturesSize);
};

typedef FApplePlatformStackWalk FPlatformStackWalk;
