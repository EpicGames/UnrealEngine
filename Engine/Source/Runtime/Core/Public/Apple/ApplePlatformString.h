// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericWidePlatformString.h"
#include "Misc/AssertionMacros.h"
#include "Misc/OutputDevice.h"

static_assert(PLATFORM_TCHAR_IS_CHAR16, "TCHAR must be CHAR16.");

// Manually declaring the type def here to avoid including a full sdk header
typedef const struct __CFString * CFStringRef;

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#if PLATFORM_MAC
#include "AppleStringUtils.h"
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "AppleStringUtils.h"
#include "IOS/IOSSystemIncludes.h"
#endif

#ifdef __OBJC__
#import <Foundation/NSString.h>
#endif

class FString;

#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

/**
 * 2-byte string implementation
 */
struct FApplePlatformString : public FGenericWidePlatformString
{
	CORE_API static void CFStringToTCHAR( CFStringRef CFStr, TCHAR *TChar );
	CORE_API static CFStringRef TCHARToCFString( const TCHAR *TChar );
};

typedef FApplePlatformString FPlatformString;

// Format specifiers to be able to print values of these types correctly, for example when using UE_LOG.
// SIZE_T format specifier
#define SIZE_T_FMT "zu"
// SIZE_T format specifier for lowercase hexadecimal output
#define SIZE_T_x_FMT "zx"
// SIZE_T format specifier for uppercase hexadecimal output
#define SIZE_T_X_FMT "zX"

// SSIZE_T format specifier
#define SSIZE_T_FMT "lld"
// SSIZE_T format specifier for lowercase hexadecimal output
#define SSIZE_T_x_FMT "llx"
// SSIZE_T format specifier for uppercase hexadecimal output
#define SSIZE_T_X_FMT "llX"

// PTRINT format specifier for decimal output
#define PTRINT_FMT SSIZE_T_FMT
// PTRINT format specifier for lowercase hexadecimal output
#define PTRINT_x_FMT SSIZE_T_x_FMT
// PTRINT format specifier for uppercase hexadecimal output
#define PTRINT_X_FMT SSIZE_T_X_FMT

// UPTRINT format specifier for decimal output
#define UPTRINT_FMT "llu"
// UPTRINT format specifier for lowercase hexadecimal output
#define UPTRINT_x_FMT "llx"
// UPTRINT format specifier for uppercase hexadecimal output
#define UPTRINT_X_FMT "llX"

// int64 format specifier for decimal output
#define INT64_FMT SSIZE_T_FMT
// int64 format specifier for lowercase hexadecimal output
#define INT64_x_FMT SSIZE_T_x_FMT
// int64 format specifier for uppercase hexadecimal output
#define INT64_X_FMT SSIZE_T_X_FMT

// uint64 format specifier for decimal output
#define UINT64_FMT "llu"
// uint64 format specifier for lowercase hexadecimal output
#define UINT64_x_FMT "llx"
// uint64 format specifier for uppercase hexadecimal output
#define UINT64_X_FMT "llX"
