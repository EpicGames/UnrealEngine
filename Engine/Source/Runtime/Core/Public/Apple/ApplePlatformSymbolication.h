// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	ApplePlatformSymbolication.h: Apple platform implementation of symbolication
==============================================================================================*/

#pragma once

#include "GenericPlatform/GenericPlatformSymbolication.h"
#include "Apple/ApplePlatformStackWalk.h"
#include "Serialization/Archive.h"
#include "Templates/SharedPointer.h"

/**
 * Opaque symbol cache for improved symbolisation performance.
 */
struct FApplePlatformSymbolCache
{
	void* Buffer0;
	void* Buffer1;
};

/**
 * Apple symbol database.
 */
struct FApplePlatformSymbolDatabase
{
	CORE_API FApplePlatformSymbolDatabase();
	CORE_API FApplePlatformSymbolDatabase(FApplePlatformSymbolDatabase const& Other);
	CORE_API ~FApplePlatformSymbolDatabase();
	CORE_API FApplePlatformSymbolDatabase& operator=(FApplePlatformSymbolDatabase const& Other);
	
	TSharedPtr<FGenericPlatformSymbolDatabase> GenericDB;
	FApplePlatformSymbolCache AppleDB;
};

/**
 * Apple symbol database hash.
 */
struct FApplePlatformSymbolDatabaseKeyFuncs
{
	enum { bAllowDuplicateKeys = 0 };
	typedef typename TCallTraits<FString>::ParamType KeyInitType;
	typedef typename TCallTraits<FApplePlatformSymbolDatabase>::ParamType ElementInitType;

	/**
	 * @return The key used to index the given element.
	 */
	static UE_FORCEINLINE_HINT KeyInitType GetSetKey(ElementInitType Element)
	{
		return Element.GenericDB->Signature;
	}

	/**
	 * @return True if the keys match.
	 */
	static UE_FORCEINLINE_HINT bool Matches(KeyInitType A,KeyInitType B)
	{
		return A == B;
	}

	/** Calculates a hash index for a key. */
	static UE_FORCEINLINE_HINT uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key);
	}
};

typedef TSet<FApplePlatformSymbolDatabase, FApplePlatformSymbolDatabaseKeyFuncs> FApplePlatformSymbolDatabaseSet;

/**
 * Apple platform implementation of symbolication - not async. handler safe, so don't call during crash reporting!
 */
struct CORE_API FApplePlatformSymbolication
{
	static void EnableCoreSymbolication(bool const bEnable);
	
	/**
	 * @param Architecture Specify the architecture in case of universal binary, if no architecture is given, will try X86_64 then ARM-64
	 */
	static bool LoadSymbolDatabaseForBinary(FString SourceFolder, FString Binary, FString BinarySignature, TOptional<FString> Architecture, FApplePlatformSymbolDatabase& OutDatabase);
	static bool SaveSymbolDatabaseForBinary(FString TargetFolder, FString Name, FString BinarySignature, FApplePlatformSymbolDatabase& Database);
	
	static bool SymbolInfoForStrippedSymbol(FApplePlatformSymbolDatabase const& Database, uint64 ProgramCounter, uint64 ModuleOffset, FString ModuleSignature, FProgramCounterSymbolInfo& Info);
	
	static bool SymbolInfoForAddress(uint64 ProgramCounter, FProgramCounterSymbolInfo& Info);
};

typedef FApplePlatformSymbolDatabase FPlatformSymbolDatabase;
typedef FApplePlatformSymbolication FPlatformSymbolication;
typedef TSet<FApplePlatformSymbolDatabase, FApplePlatformSymbolDatabaseKeyFuncs> FPlatformSymbolDatabaseSet;
