// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	ApplePlatformTLS.h: Apple platform TLS (Thread local storage and thread ID) functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformTLS.h"
#include "CoreTypes.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

/**
* Apple implementation of the TLS OS functions
**/
struct CORE_API FApplePlatformTLS : public FGenericPlatformTLS
{
	/**
	 * Returns the currently executing thread's id
	 */
	static uint32 GetCurrentThreadId(void);

	/**
	 * Allocates a thread local store slot
	 */
	static uint32 AllocTlsSlot(void);

	/**
	 * Sets a value in the specified TLS slot
	 *
	 * @param SlotIndex the TLS index to store it in
	 * @param Value the value to store in the slot
	 */
	static void SetTlsValue(uint32 SlotIndex,void* Value);

	/**
	 * Reads the value stored at the specified TLS slot
	 *
	 * @return the value stored in the slot
	 */
	static void* GetTlsValue(uint32 SlotIndex);

	/**
	 * Frees a previously allocated TLS slot
	 *
	 * @param SlotIndex the TLS index to store it in
	 */
	static void FreeTlsSlot(uint32 SlotIndex);
};

typedef FApplePlatformTLS FPlatformTLS;
