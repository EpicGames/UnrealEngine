// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	ApplePlatformTime.h: Apple platform Time functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformTime.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreTypes.h"
#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

/**
 * Please see following UDN post about using rdtsc on processors that support
 * result being invariant across cores.
 *
 * https://udn.epicgames.com/lists/showpost.php?id=46794&list=unprog3
 */



/**
* Apple platform implementation of the Time OS functions
**/
struct CORE_API FApplePlatformTime : public FGenericPlatformTime
{
	static double InitTiming();

	static double Seconds();

	static uint32 Cycles();

	static uint64 Cycles64();

	static FCPUTime GetCPUTime();

	static inline double GetSecondsTimeOffset()
	{
		return 16777216.0;
	}
};

typedef FApplePlatformTime FPlatformTime;
