// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include <atomic>
#include <type_traits>

#include <mach/port.h>
#include <pthread.h>
#include <os/lock.h>

namespace UE
{

namespace Private
{

typedef void LockImplementation(os_unfair_lock_t lock, os_unfair_lock_flags_t flags);

CORE_API extern LockImplementation* LockWithFlags;

} // namespace Private

class FAppleRecursiveMutex final
{
    os_unfair_lock Mutex {OS_UNFAIR_LOCK_INIT};

	static_assert(std::is_same_v<uint32, mach_port_t>);
	std::atomic<uint32_t> ThreadId = MACH_PORT_NULL;
	uint32 LockCount = 0;

public:
    FAppleRecursiveMutex() = default;

    FAppleRecursiveMutex(const FAppleRecursiveMutex&) = delete;
    FAppleRecursiveMutex& operator=(const FAppleRecursiveMutex&) = delete;

	UE_FORCEINLINE_HINT void Lock()
    {
		const uint32 CurrentThreadId = (uint32)pthread_mach_thread_np(pthread_self());

        if (ThreadId.load(std::memory_order_relaxed) != CurrentThreadId)
		{
			Private::LockWithFlags(&Mutex, OS_UNFAIR_LOCK_FLAG_ADAPTIVE_SPIN);
			ThreadId.store(CurrentThreadId, std::memory_order_relaxed);
		}
		++LockCount;
    }

	UE_FORCEINLINE_HINT bool TryLock()
    {
		const uint32 CurrentThreadId = (uint32)pthread_mach_thread_np(pthread_self());
		
		if (ThreadId.load(std::memory_order_relaxed) == CurrentThreadId)
		{
			++LockCount;
			return true;
		}

		if (os_unfair_lock_trylock(&Mutex))
		{
			ThreadId.store(CurrentThreadId, std::memory_order_relaxed);
			++LockCount;
			return true;
		}

		return false;
    }

	UE_FORCEINLINE_HINT void Unlock()
    {
		if ((--LockCount) == 0)
		{
			ThreadId.store(0, std::memory_order_relaxed);
			os_unfair_lock_unlock(&Mutex);
		}
    }
};

} // UE
