// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if PLATFORM_APPLE

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

#include "Misc/AssertionMacros.h"

#import <Foundation/NSString.h>

class FString;

@interface NSString (FString_Extensions)

/**
 * Converts an TCHAR string to an NSString
 */
+ (NSString*) stringWithTCHARString:(const TCHAR*)MyTCHARString;

/**
 * Converts an FString to an NSString
 */
+ (NSString*) stringWithFString:(const FString&)MyFString;

@end

class FAppleStringUtils
{
public:
	CORE_API static NSString* ConvertToNSString(const FString& InString);
	CORE_API static FString ConvertToFString(const  NSString* InNSString);
};

#endif