// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HAL/Platform.h"

struct tagRECT
{
	int32 left;
	int32 top;
	int32 right;
	int32 bottom;
};
typedef struct tagRECT RECT;