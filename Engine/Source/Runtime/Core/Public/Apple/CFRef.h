// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE4CFRef_Private
{
	template <class CFRefType>
	struct TTollFreeBridgeType;
	
#ifdef __OBJC__
	template<> struct TTollFreeBridgeType<CFArrayRef> { using Type = NSArray*; };
	template<> struct TTollFreeBridgeType<CFAttributedStringRef> { using Type = NSAttributedString*; };
	template<> struct TTollFreeBridgeType<CFBooleanRef> { using Type = NSNumber*; };
	template<> struct TTollFreeBridgeType<CFCalendarRef> { using Type = NSCalendar*; };
	template<> struct TTollFreeBridgeType<CFCharacterSetRef> { using Type = NSCharacterSet*; };
	template<> struct TTollFreeBridgeType<CFDataRef> { using Type = NSData*; };
	template<> struct TTollFreeBridgeType<CFDateRef> { using Type = NSDate*; };
	template<> struct TTollFreeBridgeType<CFDictionaryRef> { using Type = NSDictionary*; };
	template<> struct TTollFreeBridgeType<CFErrorRef> { using Type = NSError*; };
	template<> struct TTollFreeBridgeType<CFLocaleRef> { using Type = NSLocale*; };
	template<> struct TTollFreeBridgeType<CFMutableArrayRef> { using Type = NSMutableArray*; };
	template<> struct TTollFreeBridgeType<CFMutableAttributedStringRef> { using Type = NSMutableAttributedString*; };
	template<> struct TTollFreeBridgeType<CFMutableCharacterSetRef> { using Type = NSMutableCharacterSet*; };
	template<> struct TTollFreeBridgeType<CFMutableDataRef> { using Type = NSMutableData*; };
	template<> struct TTollFreeBridgeType<CFMutableDictionaryRef> { using Type = NSMutableDictionary*; };
	template<> struct TTollFreeBridgeType<CFMutableSetRef> { using Type = NSMutableSet*; };
	template<> struct TTollFreeBridgeType<CFMutableStringRef> { using Type = NSMutableString*; };
	template<> struct TTollFreeBridgeType<CFNullRef> { using Type = NSNull*; };
	template<> struct TTollFreeBridgeType<CFNumberRef> { using Type = NSNumber*; };
	template<> struct TTollFreeBridgeType<CFReadStreamRef> { using Type = NSInputStream*; };
	template<> struct TTollFreeBridgeType<CFRunLoopTimerRef> { using Type = NSTimer*; };
	template<> struct TTollFreeBridgeType<CFSetRef> { using Type = NSSet*; };
	template<> struct TTollFreeBridgeType<CFStringRef> { using Type = NSString*; };
	template<> struct TTollFreeBridgeType<CFTimeZoneRef> { using Type = NSTimeZone*; };
	template<> struct TTollFreeBridgeType<CFURLRef> { using Type = NSURL*; };
	template<> struct TTollFreeBridgeType<CFWriteStreamRef> { using Type = NSOutputStream*; };
#endif // __OBJC__
}

template <class CFRefType>
class TCFRef
{
public:

	/**
	 * Default constructor (initialized to null).
	 */
	TCFRef()
	: Ref(nullptr)
	{
	}

	/**
	 * Take ownership of a Core Foundation Ref without increasing its reference count.
	 *
	 * @param InRef The Ref to take ownership of.
	 */
	TCFRef(CFRefType InRef)
	: Ref(InRef)
	{
	}

	/**
	 * Copy constructor.
	 *
	 * @param Other The instance to copy.
	 */
	TCFRef(const TCFRef& Other)
	{
		CFRetain(Other.Ref);
		Ref = Other.Ref;
	}

	/**
	 * Move constructor.
	 *
	 * @param Other The instance to move.
	 */
	TCFRef(TCFRef&& Other)
	{
		Ref = Other.Ref;
		Other.Ref = nullptr;
	}

	/**
	 * Destructor.
	 */
	~TCFRef()
	{
		if (Ref)
		{
			CFRelease(Ref);
		}
	}

	/**
	 * Copy assignment operator.
	 *
	 * @param Other The instance to copy.
	 */
	TCFRef& operator=(const TCFRef& Other)
	{
		if (this != &Other)
		{
			if (Ref != nullptr)
			{
				CFRelease(Ref);
			}
			CFRetain(Other.Ref);
			Ref = Other.Ref;
		}
		return *this;
	}

	/**
	 * Move assignment operator.
	 *
	 * @param Other The instance to move.
	 */
	TCFRef& operator=(TCFRef&& Other)
	{
		if (this != &Other)
		{
			if (Ref)
			{
				CFRelease(Ref);
			}
			Ref = Other.Ref;
			Other.Ref = nullptr;
		}
		return *this;
	}

	/**
	 * Return a pointer to the Core Foundation Ref to allow it to be passed
	 *   into a function that will reassign it.
	 *
	 * @return A pointer to the Core Foundation Ref.
	 */
	CFRefType* GetForAssignment()
	{
		if (Ref != nullptr)
		{
			CFRelease(Ref);
			Ref = nullptr;
		}
		return &Ref;
	}

	/**
	 * Check to see if the Ref is non-null.
	 *
	 * @return True if the Ref is non-null.
	 */
	explicit operator bool() const
	{
		return Ref != nullptr;
	}

	/**
	 * Core Foundation Type conversion operator.
	 *
	 * @return The Core Foundation Ref.
	 */
	operator CFRefType() const
	{
		return Ref;
	}

	/**
	 * Foundation Type conversion operator if toll-free bridging exists for the Core Foundation Type.
	 *
	 * @return The Core Foundation Ref cast to its Foundation equivalent.
	 */
	template<typename T = CFRefType,
			 typename = typename UE4CFRef_Private::TTollFreeBridgeType<T>::Type>
	operator typename UE4CFRef_Private::TTollFreeBridgeType<T>::Type() const
	{
		return static_cast<typename UE4CFRef_Private::TTollFreeBridgeType<T>::Type>(Ref);
	}

private:
	/* The Core Foundation Type. Can be nullptr. */
	CFRefType Ref;
};

