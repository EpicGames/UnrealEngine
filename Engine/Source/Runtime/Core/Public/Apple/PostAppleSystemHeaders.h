// Copyright Epic Games, Inc. All Rights Reserved.

// Restore macros
#pragma pop_macro("PLATFORM_IOS")
#pragma pop_macro("PLATFORM_TVOS")
#pragma pop_macro("PLATFORM_VISIONOS")