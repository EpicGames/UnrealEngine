// Copyright Epic Games, Inc. All Rights Reserved.

// Save these macros; mach-o/loader.h redefines them
#pragma push_macro("PLATFORM_IOS")
#pragma push_macro("PLATFORM_TVOS")
#pragma push_macro("PLATFORM_VISIONOS")
