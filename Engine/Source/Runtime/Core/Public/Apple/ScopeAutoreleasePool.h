// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#import <Foundation/NSAutoreleasePool.h>

#ifdef __OBJC__
#if !__has_feature(objc_arc)

class FScopeAutoreleasePool
{
public:

	FScopeAutoreleasePool()
	{
		Pool = [[NSAutoreleasePool alloc] init];
	}

	~FScopeAutoreleasePool()
	{
		[Pool release];
	}

private:

	NSAutoreleasePool*	Pool;
};

#define SCOPED_AUTORELEASE_POOL const FScopeAutoreleasePool UE_JOIN(Pool,__LINE__);

#endif // !__has_feature(objc_arc)
#endif // __OBJC__
