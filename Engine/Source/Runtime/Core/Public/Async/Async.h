// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PlatformAffinity.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/ThreadSafeCounter.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreStats.h"
#include "Misc/Fork.h"
#include "Misc/IQueuedWork.h"
#include "Misc/QueuedThreadPool.h"
#include "Containers/Ticker.h"
#include "Stats/Stats.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

/**
 * Enumerates available asynchronous execution methods.
 */
enum class EAsyncExecution
{
	/** Execute in Task Graph (for short running tasks). */
	TaskGraph,

	/** Execute in Task Graph on the main thread (for short running tasks that are safe to run anywhere including inside other waits). 
	*   Note: this could end up running inside the GC or inside PostLoads, code with delegates that can't be known to be safe in advance
	*         should opt for TaskGraphMainTick instead.
	*/
	TaskGraphMainThread,

	/** Execute inside a Tick on the main thread (for short running tasks that needs to run at a safe place). */
	TaskGraphMainTick,

	/** Execute in separate thread if supported (for long running tasks). */
	Thread,

	/** Execute in separate thread if supported or supported post fork (see FForkProcessHelper::CreateThreadIfForkSafe) (for long running tasks). */
	ThreadIfForkSafe,

	/** Execute in global queued thread pool. */
	ThreadPool,

#if WITH_EDITOR
	/** Execute in large global queued thread pool. */
	LargeThreadPool
#endif
};


/**
 * Template for setting a promise value from a callable.
 */
template<typename ResultType, typename CallableType>
inline void SetPromise(TPromise<ResultType>& Promise, CallableType&& Callable)
{
	Promise.SetValue(Forward<CallableType>(Callable)());
}

template<typename CallableType>
inline void SetPromise(TPromise<void>& Promise, CallableType&& Callable)
{
	Forward<CallableType>(Callable)();
	Promise.SetValue();
}

/**
 * Base class for asynchronous functions that are executed in the Task Graph system.
 */
class FAsyncGraphTaskBase
{
public:

	/**
	 * Gets the task's stats tracking identifier.
	 *
	 * @return Stats identifier.
	 */
	TStatId GetStatId() const
	{
		return GET_STATID(STAT_TaskGraph_OtherTasks);
	}

	/**
	 * Gets the mode for tracking subsequent tasks.
	 *
	 * @return Always track subsequent tasks.
	 */
	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::FireAndForget;
	}
};


/**
 * Template for asynchronous functions that are executed in the Task Graph system.
 */
template<typename ResultType>
class TAsyncGraphTask
	: public FAsyncGraphTaskBase
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InFunction The function to execute asynchronously.
	 * @param InPromise The promise object used to return the function's result.
	 */
	TAsyncGraphTask(TUniqueFunction<ResultType()>&& InFunction, TPromise<ResultType>&& InPromise, ENamedThreads::Type InDesiredThread = ENamedThreads::AnyThread)
		: Function(MoveTemp(InFunction))
		, Promise(MoveTemp(InPromise))
		, DesiredThread(InDesiredThread)
	{ }

public:

	/**
	 * Performs the actual task.
	 *
	 * @param CurrentThread The thread that this task is executing on.
	 * @param MyCompletionGraphEvent The completion event.
	 */
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		SetPromise(Promise, Function);
	}

	/**
	 * Returns the name of the thread that this task should run on.
	 *
	 * @return Always run on any thread.
	 */
	ENamedThreads::Type GetDesiredThread()
	{
		return DesiredThread;
	}

	/**
	 * Gets the future that will hold the asynchronous result.
	 *
	 * @return A TFuture object.
	 */
	TFuture<ResultType> GetFuture()
	{
		return Promise.GetFuture();
	}

private:

	/** The function to execute on the Task Graph. */
	TUniqueFunction<ResultType()> Function;

	/** The promise to assign the result to. */
	TPromise<ResultType> Promise;

	/** The desired execution thread. */
	ENamedThreads::Type DesiredThread;
};


/**
 * Template for asynchronous functions that are executed in a separate thread.
 */
template<typename ResultType>
class TAsyncRunnable
	: public FRunnable
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InFunction The function to execute asynchronously.
	 * @param InPromise The promise object used to return the function's result.
	 * @param InThreadFuture The thread that is running this task.
	 */
	TAsyncRunnable(TUniqueFunction<ResultType()>&& InFunction, TPromise<ResultType>&& InPromise, TFuture<FRunnableThread*>&& InThreadFuture)
		: Function(MoveTemp(InFunction))
		, Promise(MoveTemp(InPromise))
		, ThreadFuture(MoveTemp(InThreadFuture))
	{ }

public:

	//~ FRunnable interface

	virtual uint32 Run() override;

private:

	/** The function to execute on the Task Graph. */
	TUniqueFunction<ResultType()> Function;

	/** The promise to assign the result to. */
	TPromise<ResultType> Promise;

	/** The thread running this task. */
	TFuture<FRunnableThread*> ThreadFuture;
};


/**
 * Template for asynchronous functions that are executed in the queued thread pool.
 */
template<typename ResultType>
class TAsyncQueuedWork
	: public IQueuedWork
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InFunction The function to execute asynchronously.
	 * @param InPromise The promise object used to return the function's result.
	 */
	TAsyncQueuedWork(TUniqueFunction<ResultType()>&& InFunction, TPromise<ResultType>&& InPromise)
		: Function(MoveTemp(InFunction))
		, Promise(MoveTemp(InPromise))
	{ }

public:

	//~ IQueuedWork interface

	virtual void DoThreadedWork() override
	{
		SetPromise(Promise, Function);
		delete this;
	}

	virtual void Abandon() override
	{
		// not supported
	}

private:

	/** The function to execute on the Task Graph. */
	TUniqueFunction<ResultType()> Function;

	/** The promise to assign the result to. */
	TPromise<ResultType> Promise;
};


/**
 * Helper struct used to generate unique ids for the stats.
 */
struct FAsyncThreadIndex
{
	static CORE_API int32 GetNext();
};


/**
 * Execute a given function asynchronously.
 *
 * Usage examples:
 *
 *	// using global function
 *		int TestFunc()
 *		{
 *			return 123;
 *		}
 *
 *		TUniqueFunction<int()> Task = TestFunc();
 *		auto Result = Async(EAsyncExecution::Thread, Task);
 *
 *	// using lambda
 *		TUniqueFunction<int()> Task = []()
 *		{
 *			return 123;
 *		}
 *
 *		auto Result = Async(EAsyncExecution::Thread, Task);
 *
 *
 *	// using inline lambda
 *		auto Result = Async(EAsyncExecution::Thread, []() {
 *			return 123;
 *		}
 *
 * @param CallableType The type of callable object.
 * @param Execution The execution method to use, i.e. on Task Graph or in a separate thread.
 * @param Function The function to execute.
 * @param CompletionCallback An optional callback function that is executed when the function completed execution.
 * @return A TFuture object that will receive the return value from the function.
 */
template<typename CallableType>
auto Async(EAsyncExecution Execution, CallableType&& Callable, TUniqueFunction<void()> CompletionCallback = nullptr) -> TFuture<decltype(Forward<CallableType>(Callable)())>
{
	using ResultType = decltype(Forward<CallableType>(Callable)());
	TUniqueFunction<ResultType()> Function(Forward<CallableType>(Callable));
	TPromise<ResultType> Promise(MoveTemp(CompletionCallback));
	TFuture<ResultType> Future = Promise.GetFuture();

	switch (Execution)
	{
	case EAsyncExecution::TaskGraphMainThread:
		// fallthrough
	case EAsyncExecution::TaskGraph:
		{
			TGraphTask<TAsyncGraphTask<ResultType>>::CreateTask().ConstructAndDispatchWhenReady(MoveTemp(Function), MoveTemp(Promise), Execution == EAsyncExecution::TaskGraph ? ENamedThreads::AnyThread : ENamedThreads::GameThread);
		}
		break;
	case EAsyncExecution::TaskGraphMainTick:
		FTSTicker::GetCoreTicker().AddTicker(TEXT("Async_TaskGraphMainTick"), 0.0f,
			[Function = MoveTemp(Function), Promise = MoveTemp(Promise)](float) mutable
			{
				SetPromise(Promise, Function);
				return false;
			}
		);
		break;
	case EAsyncExecution::Thread:
		if (FPlatformProcess::SupportsMultithreading())
		{
			TPromise<FRunnableThread*> ThreadPromise;
			TAsyncRunnable<ResultType>* Runnable = new TAsyncRunnable<ResultType>(MoveTemp(Function), MoveTemp(Promise), ThreadPromise.GetFuture());
			
			const FString TAsyncThreadName = FString::Printf(TEXT("TAsync %d"), FAsyncThreadIndex::GetNext());
			FRunnableThread* RunnableThread = FRunnableThread::Create(Runnable, *TAsyncThreadName);

			check(RunnableThread != nullptr);
			check(RunnableThread->GetThreadType() == FRunnableThread::ThreadType::Real);

			ThreadPromise.SetValue(RunnableThread);
		}
		else
		{
			SetPromise(Promise, Function);
		}
		break;

	case EAsyncExecution::ThreadIfForkSafe:
		if (FPlatformProcess::SupportsMultithreading() || FForkProcessHelper::IsForkedMultithreadInstance())
		{
			TPromise<FRunnableThread*> ThreadPromise;
			TAsyncRunnable<ResultType>* Runnable = new TAsyncRunnable<ResultType>(MoveTemp(Function), MoveTemp(Promise), ThreadPromise.GetFuture());

			const FString TAsyncThreadName = FString::Printf(TEXT("TAsync %d"), FAsyncThreadIndex::GetNext());
			FRunnableThread* RunnableThread = FForkProcessHelper::CreateForkableThread(Runnable, *TAsyncThreadName);

			check(RunnableThread != nullptr);
			check(RunnableThread->GetThreadType() == FRunnableThread::ThreadType::Real);

			ThreadPromise.SetValue(RunnableThread);
		}
		else
		{
			SetPromise(Promise, Function);
		}
		break;

	case EAsyncExecution::ThreadPool:
		if (FPlatformProcess::SupportsMultithreading())
		{
			check(GThreadPool != nullptr);
			GThreadPool->AddQueuedWork(new TAsyncQueuedWork<ResultType>(MoveTemp(Function), MoveTemp(Promise)));
		}
		else
		{
			SetPromise(Promise, Function);
		}
		break;

#if WITH_EDITOR
	case EAsyncExecution::LargeThreadPool:
		if (FPlatformProcess::SupportsMultithreading())
		{
			check(GLargeThreadPool != nullptr);
			GLargeThreadPool->AddQueuedWork(new TAsyncQueuedWork<ResultType>(MoveTemp(Function), MoveTemp(Promise)));
		}
		else
		{
			SetPromise(Promise, Function);
		}
		break;
#endif

	default:
		check(false); // not implemented yet!
	}

	return MoveTemp(Future);
}

/**
 * Execute a given function asynchronously on the specified thread pool.
 *
 * @param CallableType The type of callable object.
 * @param ThreadPool The thread pool to execute on.
 * @param Function The function to execute.
 * @param CompletionCallback An optional callback function that is executed when the function completed execution.
 * @result A TFuture object that will receive the return value from the function.
 */
template<typename CallableType>
auto AsyncPool(FQueuedThreadPool& ThreadPool, CallableType&& Callable, TUniqueFunction<void()> CompletionCallback = nullptr, EQueuedWorkPriority InQueuedWorkPriority = EQueuedWorkPriority::Normal) -> TFuture<decltype(Forward<CallableType>(Callable)())>
{
	using ResultType = decltype(Forward<CallableType>(Callable)());
	TUniqueFunction<ResultType()> Function(Forward<CallableType>(Callable));
	TPromise<ResultType> Promise(MoveTemp(CompletionCallback));
	TFuture<ResultType> Future = Promise.GetFuture();

	ThreadPool.AddQueuedWork(new TAsyncQueuedWork<ResultType>(MoveTemp(Function), MoveTemp(Promise)), InQueuedWorkPriority);

	return MoveTemp(Future);
}

/**
 * Execute a given function asynchronously using a separate thread.
 *
 * @param CallableType The type of callable object.
 * @param Function The function to execute.
 * @param StackSize stack space to allocate for the new thread
 * @param ThreadPri thread priority
 * @param CompletionCallback An optional callback function that is executed when the function completed execution.
 * @result A TFuture object that will receive the return value from the function.
 */
template<typename CallableType>
auto AsyncThread(CallableType&& Callable, uint32 StackSize = 0, EThreadPriority ThreadPri = TPri_Normal, TUniqueFunction<void()> CompletionCallback = nullptr) -> TFuture<decltype(Forward<CallableType>(Callable)())>
{
	using ResultType = decltype(Forward<CallableType>(Callable)());
	TUniqueFunction<ResultType()> Function(Forward<CallableType>(Callable));
	TPromise<ResultType> Promise(MoveTemp(CompletionCallback));
	TFuture<ResultType> Future = Promise.GetFuture();

	if (FPlatformProcess::SupportsMultithreading())
	{
		TPromise<FRunnableThread*> ThreadPromise;
		TAsyncRunnable<ResultType>* Runnable = new TAsyncRunnable<ResultType>(MoveTemp(Function), MoveTemp(Promise), ThreadPromise.GetFuture());

		const FString TAsyncThreadName = FString::Printf(TEXT("TAsyncThread %d"), FAsyncThreadIndex::GetNext());
		FRunnableThread* RunnableThread = FRunnableThread::Create(Runnable, *TAsyncThreadName, StackSize, ThreadPri);

		check(RunnableThread != nullptr);

		ThreadPromise.SetValue(RunnableThread);
	}
	else
	{
		SetPromise(Promise, Function);
	}

	return MoveTemp(Future);
}

/**
 * Convenience function for executing code asynchronously on the Task Graph.
 *
 * @param Thread The name of the thread to run on.
 * @param Function The function to execute.
 */
CORE_API void AsyncTask(ENamedThreads::Type Thread, TUniqueFunction<void()> Function);

/* Inline functions
 *****************************************************************************/

template<typename ResultType>
uint32 TAsyncRunnable<ResultType>::Run()
{
	SetPromise(Promise, Function);
	FRunnableThread* Thread = ThreadFuture.Get();

	// Enqueue deletion of the thread to a different thread.
	Async(EAsyncExecution::TaskGraph, [Thread, this]() {
			delete Thread;
			delete this;
		}
	);

	return 0;
}
