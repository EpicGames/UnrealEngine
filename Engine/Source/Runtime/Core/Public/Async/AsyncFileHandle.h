// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Function.h"
#include "Stats/Stats.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Templates/Function.h"

class IAsyncReadRequest;

DECLARE_MEMORY_STAT_EXTERN(TEXT("Async File Handle Memory"), STAT_AsyncFileMemory, STATGROUP_Memory, CORE_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Async File Handles"), STAT_AsyncFileHandles, STATGROUP_Memory, CORE_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Async File Requests"), STAT_AsyncFileRequests, STATGROUP_Memory, CORE_API);

// Note on threading. Like the rest of the filesystem platform abstraction, these methods are threadsafe, but it is expected you are not concurrently _using_ these data structures. 

class IAsyncReadRequest;
typedef TFunction<void(bool bWasCancelled, IAsyncReadRequest*)> FAsyncFileCallBack;

// Hacky base class to avoid 8 bytes of padding after the vtable
class IAsyncReadRequestFixLayout
{
public:
	virtual ~IAsyncReadRequestFixLayout() = default;
};

class IAsyncReadRequest : public IAsyncReadRequestFixLayout
{
protected:
	union
	{
		PTRINT Size;
		uint8* Memory;
	};
	FAsyncFileCallBack Callback;
	std::atomic<bool> bDataIsReady = false;
	std::atomic<bool> bCompleteAndCallbackCalled = false;
	std::atomic<bool> bCompleteSync = false;
	std::atomic<bool> bCanceled = false;
	const bool bSizeRequest;
	const bool bUserSuppliedMemory;
public:

	inline IAsyncReadRequest(FAsyncFileCallBack* InCallback, bool bInSizeRequest, uint8* UserSuppliedMemory)
		: bSizeRequest(bInSizeRequest)
		, bUserSuppliedMemory(!!UserSuppliedMemory)
	{
		if (InCallback)
		{
			Callback = *InCallback;
		}
		if (bSizeRequest)
		{
			Size = -1;
			check(!UserSuppliedMemory && !bUserSuppliedMemory); // size requests don't do memory
		}
		else
		{
			Memory = UserSuppliedMemory;
			check((!!Memory) == bUserSuppliedMemory);
		}
		INC_DWORD_STAT(STAT_AsyncFileRequests);
	}

	/* Not legal to destroy the request until it is complete. */
	virtual ~IAsyncReadRequest()
	{
		// check(bCompleteAndCallbackCalled && (bSizeRequest || !Memory)); // must be complete, and if it was a read request, the memory should be gone
		UE_CLOGF(!(bCompleteAndCallbackCalled.load(std::memory_order_relaxed) && (bSizeRequest || !Memory)),
			LogHAL, Fatal, "IAsyncReadRequests must not be deleted until they are completed."
		);
		DEC_DWORD_STAT(STAT_AsyncFileRequests);
	}

	/**
	* Nonblocking poll of the state of completion.
	* @return true if the request is complete
	**/
	UE_FORCEINLINE_HINT bool PollCompletion()
	{
		// fast path to avoid a barrier until the value returns true
		if (bCompleteAndCallbackCalled.load(std::memory_order_relaxed))
		{
			// Once we return true, we need a barrier to guarantee the memory
			// writes that have happened before the release barrier can be seen.
			// Because for all we know, PollCompletion() could be the only thing
			// that is used to synchronize with the writing thread.
			return bCompleteAndCallbackCalled.load(std::memory_order_acquire);
		}

		return false;
	}

	/**
	* Waits for the request to complete, but not longer than the given time limit
	* @param TimeLimitSeconds	Zero to wait forever, otherwise the maximum amount of time to wait.
	* @return true if the request is complete
	**/
	inline bool WaitCompletion(float TimeLimitSeconds = 0.0f)
	{
		if (PollCompletion())
		{
			return true;
		}
		WaitCompletionImpl(TimeLimitSeconds);
		return PollCompletion();
	}

	/**
	* Waits for the request to complete, with an additional guarantee that the second consecutive call won't ever block, which is not a case for
	* WaitCompletion().
	*/
	virtual void EnsureCompletion()
	{
		// Default implementation is the same as WaitCompletion(0.0f) except that it skips the testing of
		// PollCompletion. This is potentially slower because we do not early exit if PollCompletion is true, but it
		// provides a stronger guarantee of completion because PollCompletion can sometimes return true while
		// completion steps are still in progress.
		WaitCompletionImpl(0.0f);
	}

	/** Cancel the request. This is a non-blocking async call and so does not ensure completion! **/
	inline void Cancel()
	{
		if (!bCanceled.load(std::memory_order_relaxed))
		{
			bCanceled.store(true, std::memory_order_release);
			bDataIsReady.store(true, std::memory_order_release);

			if (!PollCompletion())
			{
				return CancelImpl();
			}
		}
	}

	/**
	* Return the size of a completed size request. Not legal to call unless the request is complete.
	* @return Returned size of the file or -1 if the file was not found, the request was canceled or other error.
	**/
	inline int64 GetSizeResults()
	{
		check(bDataIsReady.load(std::memory_order_relaxed) && bSizeRequest);
		return bCanceled.load(std::memory_order_relaxed) ? -1 : Size;
	}

	/**
	* Return the bytes of a completed read request. Not legal to call unless the request is complete.
	* @return Returned memory block which if non-null contains the bytes read. Caller owns the memory block and must call FMemory::Free on it when done. Can be null if the file was not found or could not be read or the request was cancelled, or the request had AIOP_FLAG_PRECACHE.
	**/
	inline uint8* GetReadResults()
	{
		check(bDataIsReady.load(std::memory_order_relaxed) && !bSizeRequest);
		uint8* Result = Memory;
		if (bCanceled.load(std::memory_order_relaxed) && Result)
		{
			Result = nullptr;
		}
		else
		{
			if (Memory && !bUserSuppliedMemory)
			{
				ReleaseMemoryOwnershipImpl();
			}

			Memory = nullptr;
		}
		return Result;
	}

protected:
	/**
	* Waits for the request to complete, but not longer than the given time limit
	* @param TimeLimitSeconds	Zero to wait forever, otherwise the maximum amount of time to wait.
	* @return true if the request is complete
	**/
	virtual void WaitCompletionImpl(float TimeLimitSeconds) = 0;

	/** Cancel the request. This is a non-blocking async call and so does not ensure completion! **/
	virtual void CancelImpl() = 0;

	/** Transfer ownership of Memory from the async request to the outside caller (called in response to GetReadResults).
	  * It's only relevant to Read requests, in which case the most common use is to update (decrease) the STAT_AsyncFileMemory
	  * stat which is typically incremented when async requests allocate Memory.
	  * It doesn't play any role in Size requests, so it may be left empty for them.
	  */
	virtual void ReleaseMemoryOwnershipImpl() = 0;

	void SetDataComplete()
	{
		bDataIsReady.store(true, std::memory_order_release);
		if (Callback)
		{
			Callback(bCanceled.load(std::memory_order_relaxed), this);
		}
	}

	void SetAllComplete()
	{
		bCompleteAndCallbackCalled.store(true, std::memory_order_release);
	}

	void SetComplete()
	{
		SetDataComplete();
		SetAllComplete();
	}
};

class IAsyncReadFileHandle
{
public:
	IAsyncReadFileHandle()
	{
		INC_DWORD_STAT(STAT_AsyncFileHandles);
	}
	/** Destructor, also the only way to close the file handle. It is not legal to delete an async file with outstanding requests. You must always call WaitCompletion before deleting a request. **/
	virtual ~IAsyncReadFileHandle()
	{
		DEC_DWORD_STAT(STAT_AsyncFileHandles);
	}

	/**
	* Request the size of the file. This is also essentially the existence check.
	* @param CompleteCallback		Called from an arbitrary thread when the request is complete. Can be nullptr, if non-null, must remain valid until it is called. It will always be called.
	* @return A request for the size. This is owned by the caller and must be deleted by the caller.
	**/
	virtual IAsyncReadRequest* SizeRequest(FAsyncFileCallBack* CompleteCallback = nullptr) = 0;

	/**
	* Submit an async request and/or wait for an async request
	* @param Offset					Offset into the file to start reading.
	* @param BytesToRead			number of bytes to read. If this request is AIOP_Preache, the size can be anything, even MAX_int64, otherwise the size and offset must be fully contained in the file.
	* @param PriorityAndFlags		Priority and flags of the request. If this includes AIOP_FLAG_PRECACHE, then memory will never be returned. The request should always be canceled and waited for, even for a precache request.
	* @param CompleteCallback		Called from an arbitrary thread when the request is complete. Can be nullptr, if non-null, must remain valid until it is called. It will always be called.
	* @return A request for the read. This is owned by the caller and must be deleted by the caller.
	**/
	virtual IAsyncReadRequest* ReadRequest(int64 Offset, int64 BytesToRead, EAsyncIOPriorityAndFlags PriorityAndFlags = AIOP_Normal, FAsyncFileCallBack* CompleteCallback = nullptr, uint8* UserSuppliedMemory = nullptr) = 0;

	/** Return true if this file is backed by a cache, if not, then precache requests are ignored. **/
	virtual bool UsesCache()
	{
		return true;
	}

	/** Minimizes buffers held internally by this handle. **/
	virtual void ShrinkHandleBuffers()
	{ }

	// Non-copyable
	IAsyncReadFileHandle(const IAsyncReadFileHandle&) = delete;
	IAsyncReadFileHandle& operator=(const IAsyncReadFileHandle&) = delete;
};
