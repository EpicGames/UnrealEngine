// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncFileHandle.h"


/**
 * IAsyncReadRequest returned from FAsyncReadFileHandleNull;
 * guaranteed to be a cancelled readrequest with no size or bytes when the Callback is called.
 */
class FAsyncReadRequestNull : public IAsyncReadRequest
{
public:
	FAsyncReadRequestNull(FAsyncFileCallBack* InCallback, bool bInSizeRequest)
		: IAsyncReadRequest(InCallback, bInSizeRequest, nullptr /* UserSuppliedMemory */)
	{
		bCanceled = true;
		SetComplete();
	}

protected:
	virtual void WaitCompletionImpl(float TimeLimitSeconds) override
	{
	}

	virtual void CancelImpl() override
	{
	}

	virtual void ReleaseMemoryOwnershipImpl() override
	{
	}
};

/**
 * An IAsyncReadFileHandle that returns only failed results,
 * used when a function has failed but needs to return a non-null IAsyncReadFileHandle.
 */
class FAsyncReadFileHandleNull : public IAsyncReadFileHandle
{
public:
	using IAsyncReadFileHandle::IAsyncReadFileHandle;

	virtual IAsyncReadRequest* SizeRequest(FAsyncFileCallBack* CompleteCallback = nullptr) override
	{
		return new FAsyncReadRequestNull(CompleteCallback, true /* bInSizeRequest */);
	}

	virtual IAsyncReadRequest* ReadRequest(int64 Offset, int64 BytesToRead,
		EAsyncIOPriorityAndFlags PriorityAndFlags = AIOP_Normal, FAsyncFileCallBack* CompleteCallback = nullptr,
		uint8* UserSuppliedMemory = nullptr) override
	{
		return new FAsyncReadRequestNull(CompleteCallback, false /* bInSizeRequest */);
	}

	virtual bool UsesCache()
	{
		return false;
	}
};