// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Async/Future.h"

class IAsyncProgress;
class IAsyncTask;

/**
 * Template for asynchronous return values.
 *
 * @param ResultType The return value's type.
 */
template<typename ResultType>
class TAsyncResult
{
public:

	/** Default constructor. */
	TAsyncResult() { }

	/**
	 * Creates and initializes a new instance with the given synchronous result value.
	 * This constructor is only available for referenceable result types, to make 'void' a feasible result type
	 * for this template class.
	 *
	 * @param Result The result value to set.
	 */
	template <typename ArgType,
			  typename = std::enable_if_t<std::is_same_v<ArgType, ResultType> && std::is_reference_v<const ArgType&>>>
	TAsyncResult(const ArgType& Result)
	{
		TPromise<ResultType> Promise;
		Promise.SetValue(Result);
		Future = Promise.GetFuture();
	}

	/**
	 * Creates and initializes a new instance
	 *
	 * @param InFuture The future to set.
	 * @param InProgress The progress object (optional).
	 * @param InTask The task object object (optional).
	 */
	TAsyncResult(TFuture<ResultType>&& InFuture, const TSharedPtr<IAsyncProgress>& InProgress, const TSharedPtr<IAsyncTask>& InTask)
		: Future(MoveTemp(InFuture))
		, Progress(InProgress)
		, Task(InTask)
	{ }

	/** Move constructor. */
	TAsyncResult(TAsyncResult&& Other)
		: Future(MoveTemp(Other.Future))
		, Progress(Other.Progress)
		, Task(Other.Task)
	{
		Other.Progress.Reset();
		Other.Task.Reset();
	}

public:

	/** Move assignment operator. */
	TAsyncResult& operator=(TAsyncResult&& Other)
	{
		Future = MoveTemp(Other.Future);
		Progress = Other.Progress;
		Task = Other.Task;

		Other.Progress.Reset();
		Other.Task.Reset();

		return *this;
	}

public:

	/**
	 * Gets the future that will hold the result.
	 *
	 * @return The future.
	 */
	const TFuture<ResultType>& GetFuture() const
	{
		return Future;
	}

	/**
	 * Get an object that indicates the progress of the task that is computing the result.
	 *
	 * @return Task progress object, or nullptr if not available.
	 */
	TSharedPtr<IAsyncProgress> GetProgess() const
	{
		return Progress;
	}

	/**
	 * Get the asynchronous task that is computing the result.
	 *
	 * @return Task object, or nullptr if not available.
	 */
	TSharedPtr<IAsyncTask> GetTask() const
	{
		return Task;
	}

private:

	/** Hidden copy constructor (async results cannot be copied). */
	TAsyncResult(const TAsyncResult&);

	/** Hidden copy assignment (async results cannot be copied). */
	TAsyncResult& operator=(const TAsyncResult&);

private:

	/** The future. */
	TFuture<ResultType> Future;

	/** The progress object. */
	TSharedPtr<IAsyncProgress> Progress;

	/** The task object. */
	TSharedPtr<IAsyncTask> Task;
};
