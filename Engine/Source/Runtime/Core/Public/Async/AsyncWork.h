// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AsyncWork.h: Definition of queued work classes
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Compression.h"
#include "HAL/ThreadSafeCounter.h"
#include "Stats/Stats.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "Async/InheritedContext.h"
#include "Misc/IQueuedWork.h"
#include "Misc/QueuedThreadPool.h"
#include "Async/Fundamental/Scheduler.h"

/**
	FAutoDeleteAsyncTask - template task for jobs that delete themselves when complete

	Sample code:

	class ExampleAutoDeleteAsyncTask : public FNonAbandonableTask
	{
		friend class FAutoDeleteAsyncTask<ExampleAutoDeleteAsyncTask>;

		int32 ExampleData;

		ExampleAutoDeleteAsyncTask(int32 InExampleData)
		 : ExampleData(InExampleData)
		{
		}

		void DoWork()
		{
			... do the work here
		}

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(ExampleAutoDeleteAsyncTask, STATGROUP_ThreadPoolAsyncTasks);
		}
	};

	
	void Example()
	{
		// start an example job
		(new FAutoDeleteAsyncTask<ExampleAutoDeleteAsyncTask>(5)->StartBackgroundTask();

		// do an example job now, on this thread
		(new FAutoDeleteAsyncTask<ExampleAutoDeleteAsyncTask>(5)->StartSynchronousTask();
	}

**/
template<typename TTask>
class FAutoDeleteAsyncTask : 
	private UE::FInheritedContextBase, 
	private IQueuedWork
{
	/** User job embedded in this task */
	TTask Task;

	/* Generic start function, not called directly
	 * @param bForceSynchronous if true, this job will be started synchronously, now, on this thread
	 **/
	void Start(bool bForceSynchronous, FQueuedThreadPool* InQueuedPool, EQueuedWorkPriority InPriority = EQueuedWorkPriority::Normal)
	{
		CaptureInheritedContext();
		FPlatformMisc::MemoryBarrier();
		FQueuedThreadPool* QueuedPool = InQueuedPool;
		if (bForceSynchronous)
		{
			QueuedPool = 0;
		}
		if (QueuedPool)
		{
			QueuedPool->AddQueuedWork(this, InPriority);
		}
		else
		{
			// we aren't doing async stuff
			DoWork();
		}
	}

	/**
	 * Tells the user job to do the work, sometimes called synchronously, sometimes from the thread pool. Calls the event tracker.
	 **/
	void DoWork()
	{
		{
			UE::FInheritedContextScope InheritedContextScope = RestoreInheritedContext();
			FScopeCycleCounter Scope(Task.GetStatId(), true);

			Task.DoWork();
		}
		delete this;
	}

	/**
	* Always called from the thread pool. Just passes off to DoWork
	**/
	virtual void DoThreadedWork()
	{
		DoWork();
	}

	/**
	 * Always called from the thread pool. Called if the task is removed from queue before it has started which might happen at exit.
	 * If the user job can abandon, we do that, otherwise we force the work to be done now (doing nothing would not be safe).
	 */
	virtual void Abandon(void)
	{
		if (Task.CanAbandon())
		{
			Task.Abandon();
			delete this;
		}
		else
		{
			DoWork();
		}
	}

public:
	/** Forwarding constructor. */
	template<typename...T>
	explicit FAutoDeleteAsyncTask(T&&... Args) : Task(Forward<T>(Args)...)
	{
	}

	/** 
	* Run this task on this thread, now. Will end up destroying myself, so it is not safe to use this object after this call.
	**/
	void StartSynchronousTask()
	{
		Start(true, nullptr);
	}

	/** 
	* Run this task on the lo priority thread pool. It is not safe to use this object after this call.
	**/
	void StartBackgroundTask(FQueuedThreadPool* InQueuedPool = GThreadPool, EQueuedWorkPriority InPriority = EQueuedWorkPriority::Normal)
	{
		Start(false, InQueuedPool, InPriority);
	}
};


/**
	FAsyncTask - template task for jobs queued to thread pools

	Sample code:

	class ExampleAsyncTask : public FNonAbandonableTask
	{
		friend class FAsyncTask<ExampleAsyncTask>;

		int32 ExampleData;

		ExampleAsyncTask(int32 InExampleData)
		 : ExampleData(InExampleData)
		{
		}

		void DoWork()
		{
			... do the work here
		}

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(ExampleAsyncTask, STATGROUP_ThreadPoolAsyncTasks);
		}
	};

	void Example()
	{

		//start an example job

		FAsyncTask<ExampleAsyncTask>* MyTask = new FAsyncTask<ExampleAsyncTask>( 5 );
		MyTask->StartBackgroundTask();

		//--or --

		MyTask->StartSynchronousTask();

		//to just do it now on this thread
		//Check if the task is done :

		if (MyTask->IsDone())
		{
		}

		//Spinning on IsDone is not acceptable( see EnsureCompletion ), but it is ok to check once a frame.
		//Ensure the task is done, doing the task on the current thread if it has not been started, waiting until completion in all cases.

		MyTask->EnsureCompletion();
		delete Task;
	}
**/

class FAsyncTaskBase
	: private UE::FInheritedContextBase
	, private IQueuedWork
{
	/** Thread safe counter that indicates WORK completion, no necessarily finalization of the job */
	FThreadSafeCounter	WorkNotFinishedCounter;
	/** If we aren't doing the work synchronously, this will hold the completion event */
	FEvent*				DoneEvent = nullptr;
	/** Pool we are queued into, maintained by the calling thread */
	FQueuedThreadPool*	QueuedPool = nullptr;
	/** Current priority */
	EQueuedWorkPriority Priority = EQueuedWorkPriority::Normal;
	/** Current flags */
	EQueuedWorkFlags Flags = EQueuedWorkFlags::None;
	/** Approximation of the peak memory (in bytes) this task could require during it's execution. */
	int64 RequiredMemory = -1;
	/** Text to identify the Task; used for debug/log purposes only. */
	const TCHAR * DebugName = nullptr;
	/** StatId used for FScopeCycleCounter */
	TStatId StatId;

	/* Internal function to destroy the completion event
	**/
	void DestroyEvent()
	{
		FPlatformProcess::ReturnSynchEventToPool(DoneEvent);
		DoneEvent = nullptr;
	}

	EQueuedWorkFlags GetQueuedWorkFlags() const final
	{
		return Flags;
	}

	/* Generic start function, not called directly
		* @param bForceSynchronous if true, this job will be started synchronously, now, on this thread
	**/
	void Start(bool bForceSynchronous, FQueuedThreadPool* InQueuedPool, EQueuedWorkPriority InQueuedWorkPriority, EQueuedWorkFlags InQueuedWorkFlags, int64 InRequiredMemory, const TCHAR * InDebugName)
	{
		CaptureInheritedContext();

		FScopeCycleCounter Scope(StatId, true);
		DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FAsyncTask::Start" ), STAT_FAsyncTask_Start, STATGROUP_ThreadPoolAsyncTasks );
		// default arg has InRequiredMemory == -1
		RequiredMemory = InRequiredMemory;
		DebugName = InDebugName;

		FPlatformMisc::MemoryBarrier();
		CheckIdle();  // can't start a job twice without it being completed first
		WorkNotFinishedCounter.Increment();
		QueuedPool = InQueuedPool;
		Priority = InQueuedWorkPriority;
		Flags = InQueuedWorkFlags;
		if (bForceSynchronous)
		{
			QueuedPool = 0;
		}
		if (QueuedPool)
		{
			if (!DoneEvent)
			{
				DoneEvent = FPlatformProcess::GetSynchEventFromPool(true);
			}
			DoneEvent->Reset();
			QueuedPool->AddQueuedWork(this, InQueuedWorkPriority);
		}
		else 
		{
			// we aren't doing async stuff
			DestroyEvent();
			DoWork();
		}
	}

	/** 
	* Tells the user job to do the work, sometimes called synchronously, sometimes from the thread pool. Calls the event tracker.
	**/
	void DoWork()
	{
		{
			UE::FInheritedContextScope InheritedContextScope = RestoreInheritedContext();
			FScopeCycleCounter Scope(StatId, true);

			DoTaskWork();
		}
		check(WorkNotFinishedCounter.GetValue() == 1);
		WorkNotFinishedCounter.Decrement();
	}

	/** 
	* Triggers the work completion event, only called from a pool thread
	**/
	void FinishThreadedWork()
	{
		check(QueuedPool);
		if (DoneEvent)
		{
			FScopeCycleCounter Scope(StatId, true);
			DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FAsyncTask::FinishThreadedWork" ), STAT_FAsyncTask_FinishThreadedWork, STATGROUP_ThreadPoolAsyncTasks );		
			DoneEvent->Trigger();
		}
	}

	/** 
	* Performs the work, this is only called from a pool thread.
	**/
	void DoThreadedWork() final
	{
		DoWork();
		FinishThreadedWork();
	}

	/**
	 * Always called from the thread pool. Called if the task is removed from queue before it has started which might happen at exit.
	 * If the user job can abandon, we do that, otherwise we force the work to be done now (doing nothing would not be safe).
	 */
	void Abandon() final
	{
		if (TryAbandonTask())
		{
			check(WorkNotFinishedCounter.GetValue() == 1);
			WorkNotFinishedCounter.Decrement();
		}
		else
		{
			DoWork();
		}
		FinishThreadedWork();
	}

	/**
	* Internal call to synchronize completion between threads, never called from a pool thread
	* @param bIsLatencySensitive specifies if waiting for the task should return as soon as possible even if this delays other tasks
	**/
	void SyncCompletion(bool bIsLatencySensitive)
	{
		FPlatformMisc::MemoryBarrier();
		if (QueuedPool)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncTask::SyncCompletion);
			FScopeCycleCounter Scope(StatId);
			DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FAsyncTask::SyncCompletion"), STAT_FAsyncTask_SyncCompletion, STATGROUP_ThreadPoolAsyncTasks);

			check(DoneEvent); // if it is not done yet, we must have an event
			DoneEvent->Wait();
			QueuedPool = 0;
		}
		CheckIdle();
	}

protected:
	void Init(TStatId InStatId)
	{
		StatId = InStatId;
	}

	/** 
	* Internal call to assert that we are idle
	**/
	void CheckIdle() const
	{
		check(WorkNotFinishedCounter.GetValue() == 0);
		if (QueuedPool) FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		check(!QueuedPool);
	}

	/** Perform task's work */
	virtual void DoTaskWork() = 0;

	/** 
	* Abandon task if possible, returns true on success, false otherwise.
	**/
	virtual bool TryAbandonTask() = 0;

	/**
	* Mark the task as canceled (i.e. no longer needed).
	**/
	virtual void MarkAsCanceled()
	{
	}

public:
	/** Destructor, not legal when a task is in process */
	virtual ~FAsyncTaskBase()
	{
		// destroying an unfinished task is a bug
		CheckIdle();
		DestroyEvent();
	}

	/**
	 * Returns an approximation of the peak memory (in bytes) this task could require during it's execution.
	 **/
	int64 GetRequiredMemory() const final
	{
		return RequiredMemory;
	}
	
	const TCHAR * GetDebugName() const final
	{
		return DebugName;
	}

	/** 
	* Run this task on this thread
	* @param bDoNow if true then do the job now instead of at EnsureCompletion
	**/
	void StartSynchronousTask(EQueuedWorkPriority InQueuedWorkPriority = EQueuedWorkPriority::Normal, EQueuedWorkFlags InQueuedWorkFlags = EQueuedWorkFlags::None, int64 InRequiredMemory = -1, const TCHAR * InDebugName = nullptr)
	{
		Start(true, GThreadPool, InQueuedWorkPriority, InQueuedWorkFlags, InRequiredMemory, InDebugName);
	}

	/** 
	* Queue this task for processing by the background thread pool
	**/
	void StartBackgroundTask(FQueuedThreadPool* InQueuedPool = GThreadPool, EQueuedWorkPriority InQueuedWorkPriority = EQueuedWorkPriority::Normal, EQueuedWorkFlags InQueuedWorkFlags = EQueuedWorkFlags::None, int64 InRequiredMemory = -1, const TCHAR * InDebugName = nullptr)
	{
		Start(false, InQueuedPool, InQueuedWorkPriority, InQueuedWorkFlags, InRequiredMemory, InDebugName);
	}

	/** 
	* Wait until the job is complete
	* @param bDoWorkOnThisThreadIfNotStarted if true and the work has not been started, retract the async task and do it now on this thread
	* @param specifies if waiting for the task should return as soon as possible even if this delays other tasks
	**/
	void EnsureCompletion(bool bDoWorkOnThisThreadIfNotStarted = true, bool bIsLatencySensitive = false)
	{
		bool DoSyncCompletion = true;
		if (bDoWorkOnThisThreadIfNotStarted)
		{
			if (QueuedPool)
			{
				if (QueuedPool->RetractQueuedWork(this))
				{
					// we got the job back, so do the work now and no need to synchronize
					DoSyncCompletion = false;
					DoWork(); 
					FinishThreadedWork();
					QueuedPool = 0;
				}
			}
			else if (WorkNotFinishedCounter.GetValue())  // in the synchronous case, if we haven't done it yet, do it now
			{
				DoWork(); 
			}
		}
		if (DoSyncCompletion)
		{
			SyncCompletion(bIsLatencySensitive);
		}
		CheckIdle(); // Must have had bDoWorkOnThisThreadIfNotStarted == false and needed it to be true for a synchronous job
	}
	
	/**
	* If not already being processed, will be rescheduled on given thread pool and priority.
	* @return true if the reschedule was successful, false if was already being processed.
	**/
	bool Reschedule(FQueuedThreadPool* InQueuedPool = GThreadPool, EQueuedWorkPriority InQueuedWorkPriority = EQueuedWorkPriority::Normal)
	{
		if (QueuedPool)
		{
			if (QueuedPool->RetractQueuedWork(this))
			{
				QueuedPool = InQueuedPool;
				Priority = InQueuedWorkPriority;
				QueuedPool->AddQueuedWork(this, InQueuedWorkPriority);
				
				return true;
			}
		}
		return false;
	}

	/**
	* Cancel the task, if possible.
	* Note that this is different than abandoning (which is called by the thread pool at shutdown). 
	* @return true if the task was canceled and is safe to delete. If it wasn't canceled, it may be done, but that isn't checked here.
	**/
	bool Cancel()
	{
		if (QueuedPool)
		{
			if (QueuedPool->RetractQueuedWork(this))
			{
				check(WorkNotFinishedCounter.GetValue() == 1);
				WorkNotFinishedCounter.Decrement();
				FinishThreadedWork();
				QueuedPool = 0;
				return true;
			}

			if (!IsWorkDone())
			{
				MarkAsCanceled();
			}
		}
		return false;
	}

	/**
	* Wait until the job is complete, up to a time limit
	* @param TimeLimitSeconds Must be positive, otherwise polls -- same as calling IsDone()
	* @return true if the task is completed
	**/
	bool WaitCompletionWithTimeout(float TimeLimitSeconds)
	{
		if (TimeLimitSeconds <= 0.0f)
		{
			return IsDone();
		}

		FPlatformMisc::MemoryBarrier();
		if (QueuedPool)
		{
			FScopeCycleCounter Scope(StatId);
			DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FAsyncTask::SyncCompletion"), STAT_FAsyncTask_SyncCompletion, STATGROUP_ThreadPoolAsyncTasks);

			uint32 Ms = uint32(TimeLimitSeconds * 1000.0f) + 1;
			check(Ms);

			check(DoneEvent); // if it is not done yet, we must have an event
			if (DoneEvent->Wait(Ms))
			{
				QueuedPool = 0;
				CheckIdle();
				return true;
			}
			return false;
		}
		CheckIdle();
		return true;
	}

	/** Returns true if the work and TASK has completed, false while it's still in progress. 
	 * prior to returning true, it synchronizes so the task can be destroyed or reused
	 */
	bool IsDone()
	{
		if (!IsWorkDone())
		{
			return false;
		}
		SyncCompletion(/*bIsLatencySensitive = */false);
		return true;
	}

	/** Returns true if the work has completed, false while it's still in progress. 
	 * This does not block and if true, you can use the results.
	 * But you can't destroy or reuse the task without IsDone() being true or EnsureCompletion()
	*/
	bool IsWorkDone() const
	{
		if (WorkNotFinishedCounter.GetValue())
		{
			return false;
		}
		return true;
	}

	/** Returns true if the work has not been started or has been completed. 
	 * NOT to be used for synchronization, but great for check()'s 
	 */
	bool IsIdle() const
	{
		return WorkNotFinishedCounter.GetValue() == 0 && QueuedPool == 0;
	}

	bool SetPriority(EQueuedWorkPriority QueuedWorkPriority)
	{
		return Reschedule(QueuedPool, QueuedWorkPriority);
	}

	EQueuedWorkPriority GetPriority() const
	{
		return Priority;
	}
};

template<typename TTask>
class FAsyncTask
	: public FAsyncTaskBase
{
	/** User job embedded in this task */ 
	TTask Task;

	struct CMarkAsCanceled
	{
		template <typename TaskType>
		auto Requires(TaskType& Task) -> decltype(
			Task.MarkAsCanceled()
		);
	};

	/**
	* Optionally mark the task as canceled if supported.
	**/
	void MarkAsCanceled() final
	{
		if constexpr (TModels_V<CMarkAsCanceled, TTask>)
		{
			Task.MarkAsCanceled();
		}
	}
public:
	FAsyncTask()
		: Task()
	{
		// Cache the StatId to remain backward compatible with TTask that declare GetStatId as non-const.
		Init(Task.GetStatId());
	}

	/** Forwarding constructor. */
	template <typename Arg0Type, typename... ArgTypes>
	FAsyncTask(Arg0Type&& Arg0, ArgTypes&&... Args)
		: Task(Forward<Arg0Type>(Arg0), Forward<ArgTypes>(Args)...)
	{
		// Cache the StatId to remain backward compatible with TTask that declare GetStatId as non-const.
		Init(Task.GetStatId());
	}

	/* Retrieve embedded user job, not legal to call while a job is in process
	* @return reference to embedded user job
	**/
	TTask& GetTask()
	{
		CheckIdle();  // can't modify a job without it being completed first
		return Task;
	}

	/* Retrieve embedded user job, not legal to call while a job is in process
	* @return reference to embedded user job
	**/
	const TTask& GetTask() const
	{
		CheckIdle();  // could be safe, but I won't allow it anyway because the data could be changed while it is being read
		return Task;
	}

	bool TryAbandonTask() final
	{
		if (Task.CanAbandon())
		{
			Task.Abandon();
			return true;
		}

		return false;
	}

	void DoTaskWork() final
	{
		Task.DoWork();
	}
};

/**
 * Stub class to use a base class for tasks that cannot be abandoned
 */
class FNonAbandonableTask
{
public:
	bool CanAbandon()
	{
		return false;
	}
	void Abandon()
	{
	}
};
