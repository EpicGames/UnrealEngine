// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM.h"
#include "CoreTypes.h"

#include <atomic>

namespace UE::Private
{

CORE_API void CheckRefCountIsNonZero();

enum class ERefCountIsQueryable : bool
{
	Yes = true,
	No = false,
};

template <typename IntType, ERefCountIsQueryable Queryable> struct TQueryableTransactionalType;

template <typename IntType> struct TQueryableTransactionalType<IntType, ERefCountIsQueryable::Yes>
{
	TQueryableTransactionalType(IntType Init)
		: Payload(Init)
	{
	}

	IntType operator++(int)
	{
		return Payload++;
	}

	IntType operator--(int)
	{
		return Payload--;
	}

	operator IntType() const
	{
		return Payload;
	}
private:
	IntType Payload = 0;
};

template <typename IntType> struct TQueryableTransactionalType<IntType, ERefCountIsQueryable::No>
{
	TQueryableTransactionalType(IntType Init)
	{
	}

	operator IntType() const
	{
		return 0;
	}
};

/**
 * TAtomicRefCount manages a transactionally-safe atomic refcount value.
 *
 * This is used by FRefCountedObject and TRefCountingMixin (in thread-safe mode).
 */
template <typename IntType, ERefCountIsQueryable Queryable = ERefCountIsQueryable::Yes>
class TAtomicRefCount
{
public:
	TAtomicRefCount() = default;

	explicit TAtomicRefCount(IntType InitialRefCount)
		: RefCount(InitialRefCount)
	{
	}

	template <auto DeleteFn>
	IntType AddRef() const
	{
		if (!AutoRTFM::IsClosed())
		{
			return ++RefCount;
		}

		IntType Refs;

		UE_AUTORTFM_OPEN
		{
			Refs = RefCount.fetch_add(1, std::memory_order_relaxed);
		};

		// As we are inside a transaction, the rollback must undo our refcount bump.
		// In general, this is best handled by Release(). However, there is one case 
		// that needs to be handled with special care. A brand-new object has a
		// refcount of zero, and a rollback must return it to this zero-refcount state.
		// However, calling AddRef() followed by Release() would not accomplish this;
		// instead, it would free the object entirely! We need to guard against this,
		// since it could lead to a double-free, so we detect the zero-reference case
		// and special-case it.
		if (Refs == 0)
		{
			UE_AUTORTFM_ONABORT(this)
			{
				RefCount.fetch_sub(1, std::memory_order_release);
				// The refcount is likely zero now, but leaving the object alive isn't a leak.
				// We are restoring the object back to its initial "just-created" state.
			};
		}
		else
		{
			UE_AUTORTFM_ONABORT(this)
			{
				Release<DeleteFn>();
			};
		}

		return Refs + 1 - TransactionallyDeferredReleaseCount;
	}

	template <auto DeleteFn>
	IntType Release() const
	{
	#if DO_GUARD_SLOW
		if (RefCount.load(std::memory_order_relaxed) == 0)
		{
			CheckRefCountIsNonZero();
		}
	#endif

		if (AutoRTFM::IsClosed())
		{
			IntType Refs;

			UE_AUTORTFM_OPEN
			{
				if constexpr (Queryable == ERefCountIsQueryable::Yes)
				{
					TransactionallyDeferredReleaseCount++;
				}
				Refs = RefCount.load(std::memory_order_relaxed) - TransactionallyDeferredReleaseCount;
			};

			if constexpr (Queryable == ERefCountIsQueryable::Yes)
			{
				UE_AUTORTFM_ONABORT(this)
				{
				#if DO_GUARD_SLOW
					if (TransactionallyDeferredReleaseCount == 0)
					{
						CheckRefCountIsNonZero();
					}
				#endif
					TransactionallyDeferredReleaseCount--;
				};
			}

			// Refcount changes and frees are deferred until the transaction is concluded.
			UE_AUTORTFM_ONCOMMIT(this)
			{
				if constexpr (Queryable == ERefCountIsQueryable::Yes)
				{
				#if DO_GUARD_SLOW
					if (TransactionallyDeferredReleaseCount == 0)
					{
						CheckRefCountIsNonZero();
					}
				#endif
					TransactionallyDeferredReleaseCount--;
				}
				ImmediatelyRelease<DeleteFn>();
			};

			return Refs;
		}
		else
		{
			return ImmediatelyRelease<DeleteFn>() - 1;
		}
	}

	// This is equivalent to https://en.cppreference.com/w/cpp/memory/shared_ptr/use_count
	// This loads with a relaxed memory ordering because a 'live' reference count is
	// unstable by nature and so there's no benefit to try and enforce memory ordering
	// around the reading of it.
	IntType GetRefCount() const
		requires (Queryable == ERefCountIsQueryable::Yes)
	{
		if (AutoRTFM::IsClosed())
		{
			IntType Refs;
			UE_AUTORTFM_OPEN
			{
				Refs = RefCount.load(std::memory_order_relaxed) - TransactionallyDeferredReleaseCount;
			};
			return Refs;
		}
		else
		{
			return RefCount.load(std::memory_order_relaxed);
		}
	}

private:
	template <auto DeleteFn>
	IntType ImmediatelyRelease() const
	{
		// fetch_sub returns the refcount _before_ it was decremented. std::memory_order_acq_rel is
		// used so that, if we do end up executing the destructor, it's not possible for side effects 
		// from executing the destructor to end up being visible before we've determined that the 
		// reference count is actually zero.
		IntType RefsBeforeRelease = RefCount.fetch_sub(1, std::memory_order_acq_rel);

	#if DO_GUARD_SLOW
		// A check-failure is issued if an object is over-released.
		if (RefsBeforeRelease == 0)
		{
			CheckRefCountIsNonZero();
		}
	#endif

		// We immediately free the object if its refcount has become zero.
		if (RefsBeforeRelease == 1)
		{
			DeleteFn(const_cast<TAtomicRefCount*>(this));
		}

		return RefsBeforeRelease;
	}

	mutable std::atomic<IntType> RefCount = 0;
	UE_NO_UNIQUE_ADDRESS mutable TQueryableTransactionalType<IntType, Queryable> TransactionallyDeferredReleaseCount = 0;
};

} // UE::Private
