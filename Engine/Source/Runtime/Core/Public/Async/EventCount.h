// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/ParkingLot.h"
#include <atomic>
#include <type_traits>

namespace UE
{

template <typename CounterType>
class TEventCount;

/** A token used to wait on TEventCount. */
template <typename CounterType>
class TEventCountToken
{
	static_assert(std::is_unsigned_v<CounterType>);

public:
	/** Returns true if the token has been assigned by PrepareWait(). */
	inline explicit operator bool() const { return !(Value & 1); }

private:
	/** Defaults to an odd value, which is never valid to wait on. */
	CounterType Value = 1;

	friend TEventCount<CounterType>;
};

/**
 * A type of event that avoids missed notifications by maintaining a notification count.
 *
 * This type of event is suited to waiting on another thread conditionally.
 * Typical usage looks similar to this example:
 *
 *     FEventCount Event;
 *     std::atomic<uint32> CurrentValue = 0;
 *
 * On the waiting thread:
 *
 *     FEventCountToken Token = Event.PrepareWait();
 *     if (CurrentValue < TargetValue)
 *     {
 *         Event.Wait(Token);
 *     }
 *
 * On the notifying thread:
 *
 *     if (++CurrentValue == TargetValue)
 *     {
 *         Event.Notify();
 *     }
 *
 * Acquiring a token before checking the condition avoids a race because Wait returns immediately
 * when the token no longer matches the notification count.
 */
template <typename CounterType>
class TEventCount
{
	static_assert(std::is_unsigned_v<CounterType>);

public:
	constexpr TEventCount() = default;

	TEventCount(const TEventCount&) = delete;
	TEventCount& operator=(const TEventCount&) = delete;

	/**
	 * Prepare to wait.
	 *
	 * Call this before any logic that must re-execute if the event is notified in the meantime.
	 *
	 * @return A token to pass to one of the wait functions to abort the wait if the event has been notified since.
	 */
	inline TEventCountToken<CounterType> PrepareWait()
	{
		TEventCountToken<CounterType> Token;

#if PLATFORM_CPU_X86_FAMILY
		// Doing a relaxed load here because fetch_or on x86 cannot return the previous value
		// so when we use the return value of fetch_or, it gets compiled into a compare_exchange.
		// The worst that can happen here is that we get a stale token value and end up 
		// not waiting for an iteration. We would then get the proper value on the next iteration.
		Token.Value = Count.load(std::memory_order_relaxed) & ~CounterType(1);
		Count.fetch_or(1, std::memory_order_acq_rel);
#else
		Token.Value = Count.fetch_or(1, std::memory_order_acq_rel) & ~CounterType(1);
#endif
		return Token;
	}

	/**
	 * Wait until the event is notified. Returns immediately if notified since the token was acquired.
	 *
	 * @param Compare   A token acquired from PrepareWait() before checking the conditions for this wait.
	 */
	inline void Wait(TEventCountToken<CounterType> Compare)
	{
		if ((Count.load(std::memory_order_acquire) & ~CounterType(1)) == Compare.Value)
		{
			ParkingLot::Wait(&Count, [this, Compare] { return (Count.load(std::memory_order_acquire) & ~CounterType(1)) == Compare.Value; }, nullptr);
		}
	}

	/**
	 * Wait until the event is notified. Returns immediately if notified since the token was acquired.
	 *
	 * @param Compare    A token acquired from PrepareWait() before checking the conditions for this wait.
	 * @param WaitTime   Relative time after which waiting is automatically canceled and the thread wakes.
	 * @return True if the event was notified before the wait time elapsed, otherwise false.
	 */
	inline bool WaitFor(TEventCountToken<CounterType> Compare, FMonotonicTimeSpan WaitTime)
	{
		if ((Count.load(std::memory_order_acquire) & ~CounterType(1)) == Compare.Value)
		{
			const ParkingLot::FWaitState WaitState = ParkingLot::WaitFor(&Count, [this, Compare] { return (Count.load(std::memory_order_acquire) & ~CounterType(1)) == Compare.Value; }, nullptr, WaitTime);

			// Make sure we return true if we did wait but also if the wait was skipped because the value actually changed before we had the opportunity to wait.
			return WaitState.bDidWake || !WaitState.bDidWait;
		}
		return true;
	}

	/**
	 * Wait until the event is notified. Returns immediately if notified since the token was acquired.
	 *
	 * @param Compare    A token acquired from PrepareWait() before checking the conditions for this wait.
	 * @param WaitTime   Absolute time after which waiting is automatically canceled and the thread wakes.
	 * @return True if the event was notified before the wait time elapsed, otherwise false.
	 */
	inline bool WaitUntil(TEventCountToken<CounterType> Compare, FMonotonicTimePoint WaitTime)
	{
		if ((Count.load(std::memory_order_acquire) & ~CounterType(1)) == Compare.Value)
		{
			const ParkingLot::FWaitState WaitState = ParkingLot::WaitUntil(&Count, [this, Compare] { return (Count.load(std::memory_order_acquire) & ~CounterType(1)) == Compare.Value; }, nullptr, WaitTime);

			// Make sure we return true if we did wait but also if the wait was skipped because the value actually changed before we had the opportunity to wait.
			return WaitState.bDidWake || !WaitState.bDidWait;
		}
		return true;
	}

	/**
	 * Notifies all waiting threads.
	 *
	 * Any threads that have called PrepareWait() and not yet waited will be notified immediately
	 * if they do wait on a token from a call to PrepareWait() that preceded this call.
	 */
	inline void Notify()
	{
		//
		// .fetch_add(0, acq_rel) is used to have a StoreLoad barrier,
		// which we can't express in C++. That works by making the load
		// also be store (via RMW) and relying on a StoreStore barrier to
		// get the desired ordering.
		//
		// Previously, this code was:
		//   CounterType Value = Count.load(std::memory_order_relaxed);
		//
		// which had a memory re-ordering and stale values being read,
		// leading to a missed Wake and dead-locked waiter, as a result.
		//
		CounterType Value = Count.fetch_add(0, std::memory_order_acq_rel);
		if ((Value & 1) && Count.compare_exchange_strong(Value, Value + 1, std::memory_order_release))
		{
			ParkingLot::WakeAll(&Count);
		}
	}

	/**
	 * Notifies all waiting threads.
	 *
	 * Any threads that have called PrepareWait() and not yet waited will be notified immediately
	 * if they do wait on a token from a call to PrepareWait() that preceded this call.
         * Note: This version doesn't provide a memory barrier, you are responsible
         *       for the memory ordering for the value you're synchronizing this eventcount with.
	 */
	inline void NotifyWeak()
	{
		// We probably don't need the weakly consistent case anymore
		// for this weak version of notify, but will need more tests before removing.
#if PLATFORM_WEAKLY_CONSISTENT_MEMORY
		//
		// .fetch_add(0, acq_rel) is used to have a StoreLoad barrier,
		// which we can't express in C++. That works by making the load
		// also be store (via RMW) and relying on a StoreStore barrier to
		// get the desired ordering.
		//
		// Previously, this code was:
		//   CounterType Value = Count.load(std::memory_order_relaxed);
		//
		// which had a memory re-ordering and stale values being read,
		// leading to a missed Wake and dead-locked waiter, as a result.
		//
		CounterType Value = Count.fetch_add(0, std::memory_order_acq_rel);
#else
		// On x86 and other non weak memory model, the fetch_or inside PrepareWait
		// is a serializing instruction that will flush the store buffer. 
		// We can omit the expensive locked op here and just do a relaxed read.
		CounterType Value = Count.load(std::memory_order_relaxed);
#endif
		if ((Value & 1) && Count.compare_exchange_strong(Value, Value + 1, std::memory_order_release))
		{
			ParkingLot::WakeAll(&Count);
		}
	}

private:
	std::atomic<CounterType> Count = 0;
};

using FEventCount = TEventCount<uint32>;
using FEventCountToken = TEventCountToken<uint32>;

} // UE
