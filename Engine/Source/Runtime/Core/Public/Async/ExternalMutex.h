// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/IntrusiveMutex.h"
#include "CoreTypes.h"

#include <atomic>

namespace UE
{

/**
 * A 2-bit mutex, with its state stored externally, that is not fair and does not support recursive locking.
 *
 * The 2 bits referenced by IsLockedFlag and MayHaveWaitingLockFlag must be initialized to 0 by the owner of
 * the state prior to using it as an external mutex.
 *
 * It is valid to construct more than one TExternalMutex for a given state and to use them concurrently.
 * It is valid to use TExternalMutex exclusively as a temporary, e.g., TExternalMutex(State).Lock();
 */
template <CIntrusiveMutexParams ParamsType>
class UE_DEPRECATED(5.8, "Use TIntrusiveMutex.")
TExternalMutex final
{
	using StateType = TIntrusiveMutexStateType_T<ParamsType>;

public:
	TExternalMutex(const TExternalMutex&) = delete;
	TExternalMutex& operator=(const TExternalMutex&) = delete;

	inline constexpr explicit TExternalMutex(std::atomic<StateType>& InState)
		: State(InState)
	{
	}

	[[nodiscard]] inline bool IsLocked() const
	{
		return TIntrusiveMutex<ParamsType>::IsLocked(State);
	}

	[[nodiscard]] inline bool TryLock()
	{
		return TIntrusiveMutex<ParamsType>::TryLock(State);
	}

	inline void Lock()
	{
		TIntrusiveMutex<ParamsType>::Lock(State);
	}

	inline void Unlock()
	{
		TIntrusiveMutex<ParamsType>::Unlock(State);
	}

private:
	std::atomic<StateType>& State;
};

namespace Core::Private
{

struct FExternalMutexParams
{
	inline constexpr static uint8 IsLockedFlag = 1 << 0;
	inline constexpr static uint8 MayHaveWaitingLockFlag = 1 << 1;
};

} // Core::Private

PRAGMA_DISABLE_DEPRECATION_WARNINGS
using FExternalMutex UE_DEPRECATED(5.7, "Use TIntrusiveMutex.") = TExternalMutex<Core::Private::FExternalMutexParams>;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

} // UE
