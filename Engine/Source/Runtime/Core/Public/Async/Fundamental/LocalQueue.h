// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "Math/RandomStream.h"
#include "Experimental/Containers/FAAArrayQueue.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Async/Fundamental/Task.h"

#include <atomic>

#if AGGRESSIVE_MEMORY_SAVING
	#define LOCALQUEUEREGISTRYDEFAULTS_MAX_LOCALQUEUES 1024
	#define LOCALQUEUEREGISTRYDEFAULTS_MAX_ITEMCOUNT 512
#else
	#define LOCALQUEUEREGISTRYDEFAULTS_MAX_LOCALQUEUES 1024
	#define LOCALQUEUEREGISTRYDEFAULTS_MAX_ITEMCOUNT 1024
#endif

#if !defined(UE_ASYNC_WORK_STEALING_QUEUE_USES_RETRY_LIMIT)
#define UE_ASYNC_WORK_STEALING_QUEUE_USES_RETRY_LIMIT 0
#endif

#if UE_ASYNC_WORK_STEALING_QUEUE_USES_RETRY_LIMIT
extern CORE_API int32 GWorkStealingQueueRetryLimit;
#endif

namespace LowLevelTasks
{
namespace LocalQueue_Impl
{
template<uint32 NumItems>
class TWorkStealingQueueBase2
{
	enum class ESlotState : uintptr_t
	{
		Free  = 0, //The slot is free and items can be put there
		Taken = 1, //The slot is in the proccess of beeing stolen 
	};

protected:
	//insert an item at the head position (this can only safe on a single thread, shared with Get) 
	inline bool Put(uintptr_t Item)
	{
		checkSlow(Item != uintptr_t(ESlotState::Free));
		checkSlow(Item != uintptr_t(ESlotState::Taken));

		uint32 Idx = (Head + 1) % NumItems;
		uintptr_t Slot = ItemSlots[Idx].Value.load(std::memory_order_acquire);

		if (Slot == uintptr_t(ESlotState::Free))
		{
			ItemSlots[Idx].Value.store(Item, std::memory_order_release);
			Head++;
			checkSlow(Head % NumItems == Idx);
			return true;
		}
		return false;
	}

	//remove an item at the head position in FIFO order (this can only safe on a single thread, shared with Put) 
	inline bool Get(uintptr_t& Item)
	{
		uint32 Idx = Head % NumItems;
		uintptr_t Slot = ItemSlots[Idx].Value.load(std::memory_order_acquire);

		if (Slot > uintptr_t(ESlotState::Taken) && ItemSlots[Idx].Value.compare_exchange_strong(Slot, uintptr_t(ESlotState::Free), std::memory_order_acq_rel))
		{
			Head--;
			checkSlow((Head + 1) % NumItems == Idx);
			Item = Slot;
			return true;
		}
		return false;
	}

	//remove an item at the tail position in LIFO order (this can be done from any thread including the one that accesses the head)
	inline bool Steal(uintptr_t& Item)
	{
#if UE_ASYNC_WORK_STEALING_QUEUE_USES_RETRY_LIMIT
		int32 Retries = GWorkStealingQueueRetryLimit;
#endif

		do
		{
			uint32 IdxVer = Tail.load(std::memory_order_acquire);
			uint32 Idx = IdxVer % NumItems;
			uintptr_t Slot = ItemSlots[Idx].Value.load(std::memory_order_acquire);

			if (Slot == uintptr_t(ESlotState::Free))
			{
				// Once we find a free slot, we need to verify if it's been freed by another steal
				// so check back the Tail value to make sure it wasn't incremented since we first read the value.
				// If we don't do this, some threads might not see that other threads
				// have already stolen the slot, and will wrongly return that no more tasks are available to steal.
				if (IdxVer != Tail.load(std::memory_order_acquire))
				{
					continue; // Loop again since tail has changed
				}
				return false;
			}
			else if (Slot != uintptr_t(ESlotState::Taken))
			{
				if (ItemSlots[Idx].Value.compare_exchange_weak(Slot, uintptr_t(ESlotState::Taken), std::memory_order_acq_rel))
				{
					if(IdxVer == Tail.load(std::memory_order_acquire))
					{
						uint32 Prev = Tail.fetch_add(1, std::memory_order_release); (void)Prev;
						checkSlow(Prev % NumItems == Idx);
						ItemSlots[Idx].Value.store(uintptr_t(ESlotState::Free), std::memory_order_release);
						Item = Slot;
						return true;
					}
					ItemSlots[Idx].Value.store(Slot, std::memory_order_release);
				}
			}
#if UE_ASYNC_WORK_STEALING_QUEUE_USES_RETRY_LIMIT
			else if (Retries-- == 0)
			{
				FPlatformProcess::YieldThread();
				Retries = GWorkStealingQueueRetryLimit;
			}
#endif
		} while(true);
	}

private:
	struct FAlignedElement
	{
		alignas(PLATFORM_CACHE_LINE_SIZE * 2) std::atomic<uintptr_t> Value = {};
	};

	alignas(PLATFORM_CACHE_LINE_SIZE * 2) uint32 Head { ~0u };
	alignas(PLATFORM_CACHE_LINE_SIZE * 2) std::atomic_uint Tail { 0 };
	alignas(PLATFORM_CACHE_LINE_SIZE * 2) FAlignedElement ItemSlots[NumItems] = {};
};

template<typename Type, uint32 NumItems>
class TWorkStealingQueue2 final : protected TWorkStealingQueueBase2<NumItems>
{
	using PointerType = Type*;

public:
	inline bool Put(PointerType Item)
	{
		return TWorkStealingQueueBase2<NumItems>::Put(reinterpret_cast<uintptr_t>(Item));
	}

	inline bool Get(PointerType& Item)
	{
		return TWorkStealingQueueBase2<NumItems>::Get(reinterpret_cast<uintptr_t&>(Item));
	}

	inline bool Steal(PointerType& Item)
	{
		return TWorkStealingQueueBase2<NumItems>::Steal(reinterpret_cast<uintptr_t&>(Item));
	}
};
}

namespace Private {

enum class ELocalQueueType
{
	EBackground,
	EForeground,
};

/********************************************************************************************************************************************
 * A LocalQueueRegistry is a collection of LockFree queues that store pointers to Items, there are ThreadLocal LocalQueues with LocalItems. *
 * LocalQueues can only be Enqueued and Dequeued by the current Thread they were installed on. But Items can be stolen from any Thread      *
 * There is a global OverflowQueue than is used when a LocalQueue goes out of scope to dump all the remaining Items in                      *
 * or when a Thread has no LocalQueue installed or when the LocalQueue is at capacity. A new LocalQueue is registers itself always.         *
 * A Dequeue Operation can only be done starting from a LocalQueue, than the GlobalQueue will be checked.                                   *
 * Finally Items might get Stolen from other LocalQueues that are registered with the LocalQueueRegistry.                                   *
 ********************************************************************************************************************************************/
template<uint32 NumLocalItems = LOCALQUEUEREGISTRYDEFAULTS_MAX_ITEMCOUNT, uint32 MaxLocalQueues = LOCALQUEUEREGISTRYDEFAULTS_MAX_LOCALQUEUES>
class TLocalQueueRegistry
{
	static uint32 Rand()
	{
		uint32 State = FPlatformTime::Cycles();
		State = State * 747796405u + 2891336453u;
		State = ((State >> ((State >> 28u) + 4u)) ^ State) * 277803737u;
		return (State >> 22u) ^ State;
	}

public:
	class TLocalQueue;

private:
	using FLocalQueueType	 = LocalQueue_Impl::TWorkStealingQueue2<FTask, NumLocalItems>;
	using FOverflowQueueType = FAAArrayQueue<FTask>;
	using DequeueHazard		 = typename FOverflowQueueType::DequeueHazard;

public:
	class TLocalQueue
	{
		template<uint32, uint32>
		friend class TLocalQueueRegistry;

	public:
		TLocalQueue() = default;
			
		TLocalQueue(TLocalQueueRegistry& InRegistry, ELocalQueueType InQueueType)
		{
			Init(InRegistry, InQueueType);
		}

		void Init(TLocalQueueRegistry& InRegistry, ELocalQueueType InQueueType)
		{
			if (bIsInitialized.exchange(true, std::memory_order_relaxed))
			{
				checkf(false, TEXT("Trying to initialize local queue more than once"));
			}
			else
			{
				Registry = &InRegistry;
				QueueType = InQueueType;

				// Local queues are never unregistered, everything is shutdown at once.
				Registry->AddLocalQueue(this);
				for (int32 PriorityIndex = 0; PriorityIndex < int32(ETaskPriority::Count); ++PriorityIndex)
				{
					DequeueHazards[PriorityIndex] = Registry->OverflowQueues[PriorityIndex].getHeadHazard();
				}
			}
		}

		~TLocalQueue()
		{
			if (bIsInitialized.exchange(false, std::memory_order_relaxed))
			{
				for (int32 PriorityIndex = 0; PriorityIndex < int32(ETaskPriority::Count); PriorityIndex++)
				{
					while (true)
					{
						FTask* Item;
						if (!LocalQueues[PriorityIndex].Get(Item))
						{
							break;
						}
						Registry->OverflowQueues[PriorityIndex].enqueue(Item);
					}
				}
			}
		}

		// add an item to the local queue and overflow into the global queue if full
		// returns true if we should wake a worker
		inline void Enqueue(FTask* Item, uint32 PriorityIndex)
		{
			checkSlow(Registry);
			checkSlow(PriorityIndex < int32(ETaskPriority::Count));
			checkSlow(Item != nullptr);

			if (!LocalQueues[PriorityIndex].Put(Item))
			{
				Registry->OverflowQueues[PriorityIndex].enqueue(Item);
			}
		}

		inline FTask* StealLocal(int32 PriorityIndex)
		{
			FTask* Item;
			if (LocalQueues[PriorityIndex].Steal(Item))
			{
				return Item;
			}

			return nullptr;
		}

		inline FTask* StealLocal(bool bPermitBackgroundTasks)
		{
			const int32 MaxPriority = bPermitBackgroundTasks ? int32(ETaskPriority::Count) : int32(ETaskPriority::ForegroundCount);

			for (int32 PriorityIndex = 0; PriorityIndex < MaxPriority; ++PriorityIndex)
			{
				if (FTask* Item = StealLocal(PriorityIndex))
				{
					return Item;
				}
			}
			return nullptr;
		}

		// Check both the local and global queue for a specific priority
		inline FTask* Dequeue(int32 PriorityIndex)
		{
			FTask* Item;
			if (LocalQueues[PriorityIndex].Get(Item))
			{
				return Item;
			}

			Item = Registry->OverflowQueues[PriorityIndex].dequeue(DequeueHazards[PriorityIndex]);
			if (Item)
			{
				return Item;
			}
			
			return nullptr;
		}

		// Check both the local and global queue in priority order
		inline FTask* Dequeue(bool bPermitBackgroundTasks)
		{
			const int32 MaxPriority = bPermitBackgroundTasks ? int32(ETaskPriority::Count)   : int32(ETaskPriority::ForegroundCount);

			for (int32 PriorityIndex = 0; PriorityIndex < MaxPriority; ++PriorityIndex)
			{
				if (FTask* Item = Dequeue(PriorityIndex))
				{
					return Item;
				}
			}

			return nullptr;
		}

		inline FTask* DequeueSteal(bool bPermitBackgroundTasks)
		{
			if (CachedRandomIndex == InvalidIndex)
			{
				CachedRandomIndex = Rand();
			}

			FTask* Result = Registry->StealItem(CachedRandomIndex, CachedPriorityIndex, bPermitBackgroundTasks);
			if (Result)
			{
				return Result;
			}
			return nullptr;
		}

	private:
		static constexpr uint32    InvalidIndex = ~0u;
		FLocalQueueType            LocalQueues[uint32(ETaskPriority::Count)];
		DequeueHazard              DequeueHazards[uint32(ETaskPriority::Count)];
		TLocalQueueRegistry*       Registry = nullptr;
		uint32                     CachedRandomIndex = InvalidIndex;
		uint32                     CachedPriorityIndex = 0;
		ELocalQueueType            QueueType;
		std::atomic<bool>          bIsInitialized = false;
	};

	TLocalQueueRegistry()
	{
	}

private:
	// Add a queue to the Registry. Thread-safe.
	void AddLocalQueue(TLocalQueue* QueueToAdd)
	{
		uint32 Index = NumLocalQueues.fetch_add(1, std::memory_order_relaxed);
		UE_CLOGF(Index >= MaxLocalQueues, LowLevelTasks, Fatal, "Attempting to add more than the maximum allowed number of queues (%d)", MaxLocalQueues);

		// std::memory_order_release to make sure values are all written to the TLocalQueue before publishing.
		LocalQueues[Index].store(QueueToAdd, std::memory_order_release);
	}

	// StealItem tries to steal an Item from a Registered LocalQueue
	// Thread-safe with AddLocalQueue
	FTask* StealItem(uint32& CachedRandomIndex, uint32& CachedPriorityIndex, bool GetBackGroundTasks)
	{
		uint32 NumQueues   = NumLocalQueues.load(std::memory_order_relaxed);
		uint32 MaxPriority = GetBackGroundTasks ? int32(ETaskPriority::Count) : int32(ETaskPriority::ForegroundCount);
		CachedRandomIndex  = CachedRandomIndex % NumQueues;

		for (uint32 Index = 0; Index < NumLocalQueues; Index++)
		{
			// Test for null in case we race on reading NumLocalQueues reserved index before the pointer is set
			if (TLocalQueue* LocalQueue = LocalQueues[Index].load(std::memory_order_acquire))
			{
				for(uint32 PriorityIndex = 0; PriorityIndex < MaxPriority; PriorityIndex++)
				{
					FTask* Item;
					if (LocalQueue->LocalQueues[PriorityIndex].Steal(Item))
					{
						return Item;
					}
					CachedPriorityIndex = ++CachedPriorityIndex < MaxPriority ? CachedPriorityIndex : 0;
				}
				CachedRandomIndex = ++CachedRandomIndex < NumQueues ? CachedRandomIndex : 0;
			}
		}
		CachedPriorityIndex = 0;
		CachedRandomIndex = TLocalQueue::InvalidIndex;
		return nullptr;
	}

public:
	// enqueue an Item directy into the Global OverflowQueue
	void Enqueue(FTask* Item, uint32 PriorityIndex)
	{
		check(PriorityIndex < int32(ETaskPriority::Count));
		check(Item != nullptr);

		OverflowQueues[PriorityIndex].enqueue(Item);
	}

	// grab an Item directy from the Global OverflowQueue
	FTask* DequeueGlobal(bool GetBackGroundTasks = true)
	{
		const int32 MaxPriority = GetBackGroundTasks ? int32(ETaskPriority::Count) : int32(ETaskPriority::ForegroundCount);

		for (int32 PriorityIndex = 0; PriorityIndex < MaxPriority; ++PriorityIndex)
		{
			if (FTask* Item = OverflowQueues[PriorityIndex].dequeue())
			{
				return Item;
			}
		}
		return nullptr;
	}

	inline FTask* DequeueSteal(bool GetBackGroundTasks)
	{
		uint32 CachedRandomIndex = Rand();
		uint32 CachedPriorityIndex = 0;
		FTask* Result = StealItem(CachedRandomIndex, CachedPriorityIndex, GetBackGroundTasks);
		if (Result)
		{
			return Result;
		}
		return nullptr;
	}

	// Not thread-safe.
	void Reset()
	{
		uint32 NumQueues = NumLocalQueues.load(std::memory_order_relaxed);
		for (uint32 Index = 0; Index < NumQueues; Index++)
		{
			LocalQueues[Index].store(0, std::memory_order_relaxed);
		}

		NumLocalQueues.store(0, std::memory_order_release);
	}

private:
	FOverflowQueueType        OverflowQueues[uint32(ETaskPriority::Count)];
	std::atomic<TLocalQueue*> LocalQueues[MaxLocalQueues] { nullptr };
	std::atomic<uint32>       NumLocalQueues {0};
};

} // namespace Private

}
