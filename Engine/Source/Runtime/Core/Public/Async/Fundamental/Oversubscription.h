// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace LowLevelTasks::Private
{

class FOversubscriptionTls
{
	static thread_local bool bIsOversubscriptionAllowed;

	friend class FOversubscriptionAllowedScope;

	CORE_API static bool& GetIsOversubscriptionAllowedRef();

public:
	static bool IsOversubscriptionAllowed() { return bIsOversubscriptionAllowed; }
};

class FOversubscriptionAllowedScope
{
	UE_NONCOPYABLE(FOversubscriptionAllowedScope);

public:
	explicit FOversubscriptionAllowedScope(bool bIsOversubscriptionAllowed)
		: bValue(FOversubscriptionTls::GetIsOversubscriptionAllowedRef())
		, bPreviousValue(bValue)
	{
		bValue = bIsOversubscriptionAllowed;
	}

	~FOversubscriptionAllowedScope()
	{
		bValue = bPreviousValue;
	}

private:
	bool& bValue;
	bool bPreviousValue;
};

} // LowLevelTasks::Private

namespace LowLevelTasks
{

class FOversubscriptionScope
{
	UE_NONCOPYABLE(FOversubscriptionScope);

public:
	explicit FOversubscriptionScope(bool bCondition = true)
	{
		if (bCondition)
		{
			TryIncrementOversubscription();
		}
	}

	~FOversubscriptionScope()
	{
		if (bIncrementOversubscriptionEmitted)
		{
			DecrementOversubscription();
		}
	}

private:
	CORE_API void TryIncrementOversubscription();
	CORE_API void DecrementOversubscription();

	bool bIncrementOversubscriptionEmitted = false;
	bool bCpuBeginEventEmitted = false;
};

} // LowLevelTests
