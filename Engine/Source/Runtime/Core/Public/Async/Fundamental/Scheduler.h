// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Fundamental/Oversubscription.h"
#include "Async/Fundamental/Task.h"
#include "Async/Fundamental/TaskShared.h"
#include "Async/Fundamental/TaskDelegate.h"
#include "Async/Fundamental/WaitingQueue.h"
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/List.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "HAL/Event.h"
#include "HAL/PlatformAffinity.h"
#include "HAL/PlatformMutex.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Thread.h"
#include "LocalQueue.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Function.h"
#include "Templates/IsInvocable.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"

#include <atomic>

namespace LowLevelTasks
{
	enum class EQueuePreference
	{
		GlobalQueuePreference,
		LocalQueuePreference,
		DefaultPreference = LocalQueuePreference,
	};

	class FSchedulerTls
	{
	protected:
		class FImpl;

		using FQueueRegistry	= Private::TLocalQueueRegistry<>;
		using FLocalQueueType	= FQueueRegistry::TLocalQueue;

		enum class EWorkerType
		{
			None,
			Background,
			Foreground,
		};

		struct FTlsValues : public TIntrusiveLinkedList<FTlsValues>
		{
			FSchedulerTls* ActiveScheduler = nullptr;
			FLocalQueueType* LocalQueue = nullptr;
			EWorkerType WorkerType = EWorkerType::None;
			std::atomic<bool> bPendingWakeUp = false;
			bool        bIsStandbyWorker = false;

			inline bool IsBackgroundWorker()
			{
				return WorkerType == EWorkerType::Background;
			}

			inline bool IsStandbyWorker()
			{
				return bIsStandbyWorker;
			}

			inline void SetStandbyWorker(bool bInIsStandbyWorker)
			{
				bIsStandbyWorker = bInIsStandbyWorker;
			}

			static void* operator new(size_t Size);
			static void operator delete(void* Ptr);
		};

		struct FTlsValuesHolder
		{
			CORE_API FTlsValuesHolder();
			CORE_API ~FTlsValuesHolder();
			
			FTlsValues* TlsValues = nullptr;
		};

	private:
		static thread_local FTlsValuesHolder TlsValuesHolder;

	public:
		CORE_API bool IsWorkerThread() const;

	protected:
		CORE_API bool HasPendingWakeUp() const;

		static FTlsValues& GetTlsValuesRef();
	};

	class FScheduler final : public FSchedulerTls
	{
		UE_NONCOPYABLE(FScheduler);
		static constexpr uint32 WorkerSpinCycles = 53;

		static CORE_API FScheduler Singleton;

		// using 16 bytes here because it fits the vtable and one additional pointer
		using FConditional = TTaskDelegate<bool(), 16>;

	public: // Public Interface of the Scheduler
		inline static FScheduler& Get();

		//start number of workers where 0 is the system default
		CORE_API void StartWorkers(uint32 NumForegroundWorkers = 0, uint32 NumBackgroundWorkers = 0, FThread::EForkable IsForkable = FThread::NonForkable, EThreadPriority InWorkerPriority = EThreadPriority::TPri_Normal, EThreadPriority InBackgroundPriority = EThreadPriority::TPri_BelowNormal, uint64 InWorkerAffinity = 0, uint64 InBackgroundAffinity = 0, uint32 InWorkersStackSize = 0);
		CORE_API void StopWorkers(bool DrainGlobalQueue = true);
		CORE_API void RestartWorkers(uint32 NumForegroundWorkers = 0, uint32 NumBackgroundWorkers = 0, FThread::EForkable IsForkable = FThread::NonForkable, EThreadPriority WorkerPriority = EThreadPriority::TPri_Normal, EThreadPriority BackgroundPriority = EThreadPriority::TPri_BelowNormal, uint64 InWorkerAffinity = 0, uint64 InBackgroundAffinity = 0, uint32 InWorkersStackSize = 0);

		//try to launch the task, the return value will specify if the task was in the ready state and has been launched
		inline bool TryLaunch(FTask& Task, EQueuePreference QueuePreference = EQueuePreference::DefaultPreference, bool bWakeUpWorker = true);

		//number of instantiated workers
		inline uint32 GetNumWorkers() const;

		//maximum number of workers, including standby workers (Oversubscription)
		inline uint32 GetMaxNumWorkers() const;

		//get the worker priority set when workers were started
		inline EThreadPriority GetWorkerPriority() const { return WorkerPriority; }

		//get the background priority set when workers were started
		inline EThreadPriority GetBackgroundPriority() const { return BackgroundPriority; }

		//determine if we're currently out of workers for a given task priority
		CORE_API bool IsOversubscriptionLimitReached(ETaskPriority TaskPriority) const;

		//event that will fire when the scheduler has reached its oversubscription limit (all threads are waiting).
		//note: This event can be broadcasted from any thread so the receiver needs to be thread-safe
		//      For optimal performance, avoid binding UObjects to this event and use AddRaw/AddLambda instead.
		//      Also, what's happening inside that callback should be as brief and simple as possible (i.e. raising an event)
		CORE_API FOversubscriptionLimitReached& GetOversubscriptionLimitReachedEvent();
	public:
		FScheduler() = default;
		~FScheduler();

	private: 
		[[nodiscard]] FTask* ExecuteTask(FTask* InTask);
		TUniquePtr<FThread> CreateWorker(uint32 WorkerId, const TCHAR* Name, bool bPermitBackgroundWork = false, FThread::EForkable IsForkable = FThread::NonForkable, Private::FWaitEvent* ExternalWorkerEvent = nullptr, FSchedulerTls::FLocalQueueType* ExternalWorkerLocalQueue = nullptr, EThreadPriority Priority = EThreadPriority::TPri_Normal, uint64 InAffinity = 0, uint32 StackSize = 0);
		void WorkerMain(Private::FWaitEvent* WorkerEvent, FSchedulerTls::FLocalQueueType* ExternalWorkerLocalQueue, uint32 WaitCycles, bool bPermitBackgroundWork);
		void StandbyLoop(Private::FWaitEvent* WorkerEvent, FSchedulerTls::FLocalQueueType* ExternalWorkerLocalQueue, uint32 WaitCycles, bool bPermitBackgroundWork);
		void WorkerLoop(Private::FWaitEvent* WorkerEvent, FSchedulerTls::FLocalQueueType* ExternalWorkerLocalQueue, uint32 WaitCycles, bool bPermitBackgroundWork);
		CORE_API void LaunchInternal(FTask& Task, EQueuePreference QueuePreference, bool bWakeUpWorker);
		inline bool WakeUpWorker(bool bBackgroundWorker);
		CORE_API void IncrementOversubscription();
		CORE_API void DecrementOversubscription();
		template<typename QueueType, FTask* (QueueType::*DequeueFunction)(bool), bool bIsStandbyWorker>
		bool TryExecuteTaskFrom(Private::FWaitEvent* WaitEvent, QueueType* Queue, Private::FOutOfWork& OutOfWork, bool bPermitBackgroundWork);

		friend class FOversubscriptionScope;
	private:
		Private::FWaitingQueue                         WaitingQueue[2] = { { WorkerEvents, OversubscriptionLimitReachedEvent }, { WorkerEvents, OversubscriptionLimitReachedEvent } };
		FSchedulerTls::FQueueRegistry                  QueueRegistry;
		UE::FPlatformRecursiveMutex                    WorkerThreadsCS;
		TUniquePtr<std::atomic<FThread*>[]>            WorkerThreads;
		TAlignedArray<FSchedulerTls::FLocalQueueType>  WorkerLocalQueues;
		TAlignedArray<Private::FWaitEvent>             WorkerEvents;
		TUniquePtr<FSchedulerTls::FLocalQueueType>     GameThreadLocalQueue;
		std::atomic_uint                               ActiveWorkers { 0 };
		std::atomic_uint                               NextWorkerId { 0 };
		std::atomic<int32>                             ForegroundCreationIndex{ 0 };
		std::atomic<int32>                             BackgroundCreationIndex{ 0 };
		uint64                                         WorkerAffinity = 0;
		uint64                                         BackgroundAffinity = 0;
		uint32										   WorkerStackSize = 0;
		EThreadPriority                                WorkerPriority = EThreadPriority::TPri_Normal;
		EThreadPriority                                BackgroundPriority = EThreadPriority::TPri_BelowNormal;
		std::atomic_bool                               TemporaryShutdown{ false };
		FOversubscriptionLimitReached                  OversubscriptionLimitReachedEvent;
	};

	UE_FORCEINLINE_HINT bool TryLaunch(FTask& Task, EQueuePreference QueuePreference = EQueuePreference::DefaultPreference, bool bWakeUpWorker = true)
	{
		return FScheduler::Get().TryLaunch(Task, QueuePreference, bWakeUpWorker);
	}

	/******************
	* IMPLEMENTATION *
	******************/
	inline bool FScheduler::TryLaunch(FTask& Task, EQueuePreference QueuePreference, bool bWakeUpWorker)
	{
		if(Task.TryPrepareLaunch())
		{
			LaunchInternal(Task, QueuePreference, bWakeUpWorker);
			return true;
		}
		return false;
	}

	inline uint32 FScheduler::GetNumWorkers() const
	{
		return ActiveWorkers.load(std::memory_order_relaxed);
	}
	
	// Return the maximum number of worker threads, including Standby Workers
	inline uint32 FScheduler::GetMaxNumWorkers() const
	{
		return WorkerLocalQueues.Num();
	}

	inline bool FScheduler::WakeUpWorker(bool bBackgroundWorker)
	{
		return WaitingQueue[bBackgroundWorker].Notify() != 0;
	}

	inline FScheduler& FScheduler::Get()
	{
		return Singleton;
	}

	inline FScheduler::~FScheduler()
	{
		StopWorkers();
	}
}
