// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Misc/EnumClassFlags.h"
#include "TaskDelegate.h"
#include "HAL/Event.h"
#include "CoreTypes.h"
#include <atomic>

#define LOWLEVEL_TASK_SIZE PLATFORM_CACHE_LINE_SIZE

/** This namespace and header is used for the internals of the task system, engine code should include Tasks/Task.h instead */
namespace LowLevelTasks
{
	DECLARE_LOG_CATEGORY_EXTERN(LowLevelTasks, Log, All);

	/** Defines the order that tasks are processed in, starting with High and ending with BackgroundLow */
	enum class ETaskPriority : int8
	{
		High,
		Normal,
		Default = Normal,
		ForegroundCount,

		BackgroundHigh = ForegroundCount,
		BackgroundNormal,
		BackgroundLow,
		Count,

		Inherit, //Inherit the TaskPriority from the launching Task or the Default Priority if not launched from a Task.
	};

	inline const TCHAR* ToString(ETaskPriority Priority)
	{
		if (Priority < ETaskPriority::High || Priority >= ETaskPriority::Count)
		{
			return nullptr;
		}

		const TCHAR* TaskPriorityToStr[] =
		{
			TEXT("High"),
			TEXT("Normal"),
			TEXT("BackgroundHigh"),
			TEXT("BackgroundNormal"),
			TEXT("BackgroundLow")
		};
		return TaskPriorityToStr[(int32)Priority];
	}

	inline bool ToTaskPriority(const TCHAR* PriorityStr, ETaskPriority& OutPriority)
	{
		if (FCString::Stricmp(PriorityStr, ToString(ETaskPriority::High)) == 0)
		{
			OutPriority = ETaskPriority::High;
			return true;
		}

		if (FCString::Stricmp(PriorityStr, ToString(ETaskPriority::Normal)) == 0)
		{
			OutPriority = ETaskPriority::Normal;
			return true;
		}

		if (FCString::Stricmp(PriorityStr, ToString(ETaskPriority::BackgroundHigh)) == 0)
		{
			OutPriority = ETaskPriority::BackgroundHigh;
			return true;
		}

		if (FCString::Stricmp(PriorityStr, ToString(ETaskPriority::BackgroundNormal)) == 0)
		{
			OutPriority = ETaskPriority::BackgroundNormal;
			return true;
		}

		if (FCString::Stricmp(PriorityStr, ToString(ETaskPriority::BackgroundLow)) == 0)
		{
			OutPriority = ETaskPriority::BackgroundLow;
			return true;
		}

		return false;
	}

	/** Flags used to change the behavior of TryCancel */
	enum class ECancellationFlags : int8
	{
		None					= 0 << 0,
		TryLaunchOnSuccess		= 1 << 0, // try to launch and the continuation immediately if it was not launched yet (requires PrelaunchCancellation to work)
		PrelaunchCancellation	= 1 << 1, // allow cancellation before a task has been launched (this also allows the optimization of TryLaunchOnSuccess)
		DefaultFlags			= TryLaunchOnSuccess | PrelaunchCancellation,
	};
	ENUM_CLASS_FLAGS(ECancellationFlags)

	/** Flags defining what operations are possible on a specific task */
	enum class ETaskFlags : int8
	{
		AllowNothing		= 0 << 0,
		AllowBusyWaiting	= 1 << 0,
		AllowCancellation	= 1 << 1,
		AllowEverything		= AllowBusyWaiting | AllowCancellation,
		DefaultFlags		= AllowEverything,
	};
	ENUM_CLASS_FLAGS(ETaskFlags)

	/**
	 * The state of an individual task, following this state machine:
	 *
	 * (I)nitThread:                                                        STORE(I)----------------------CAS(C)----------------------    
	 * (C)ancelingThread:                                                      --->|         Ready        |<-->|   CanceledAndReady   |   
	 *                                                                              ----------------------      ----------------------    
	 *                                                                                        |OR(L)                      |OR(L)               
	 *                                                                                        V                           V               
	 * (L)aunchingThread:   --------------------------------------------------CAS(E)----------------------CAS(C)----------------------    
	 * (C)ancelingThread:  |                      Running                     |<---|      Scheduled       |<-->|       Canceled       |   
	 * (E)xpeditingThread:  --------------------------------------------------      ----------------------      ----------------------    
	 *                                |OR(E)                      |OR(W)                      |OR(W)                      |OR(W)               
	 *                                V                           V                           V                           V               
	 * (W)orkerThread:      ---------------------- OR(E)----------------------      ----------------------      ---------------------- 	   
	 * (E)xpeditingThread: |      Expedited       |<---|      Expediting      |    |       Running        |    |  CanceledAndRunning  |	   
	 *                      ----------------------      ----------------------      ----------------------      ----------------------   
	 *                                |OR(W,E)                                                |OR(W)                      |OR(W)		   
	 *                                V                                                       V                           V			   
	 * (W)orkerThread:      ----------------------                                  ----------------------      ----------------------    
	 * (E)xpeditingThread: |ExpeditedAndCompleted |                                |       Completed      |    | CanceledAndCompleted |   
	 *                      ----------------------                                  ----------------------      ---------------------- 
	 */
	enum class ETaskState : int8
	{
		ReadyState		=  0,
		CanceledFlag	=  1 << 0,
		ScheduledFlag	=  1 << 1,
		RunningFlag		=  1 << 2,
		ExpeditingFlag	=  1 << 3,
		ExpeditedFlag	=  1 << 4,
		CompletedFlag	=  1 << 5,										//the default state when we create a handle
		Count			= (1 << 6) - 1,

		Ready					= ReadyState,							//means the Task is ready to be launched
		CanceledAndReady		= Ready | CanceledFlag,					//means the task was canceled and is ready to be launched (it still is required to be launched)
		Scheduled				= Ready | ScheduledFlag, 				//means the task is launched and therefore queued for execution by a worker
		Canceled				= CanceledAndReady | ScheduledFlag,		//means the task was canceled and launched and therefore queued for execution by a worker (which already might be executing it's continuation)
		Running					= Scheduled | RunningFlag,				//means the task is executing it's runnable and continuation by a worker
		CanceledAndRunning		= Canceled | RunningFlag,				//means the task is executing it's continuation  but the runnable was cancelled
		Expediting				= Running | ExpeditingFlag, 			//means the task is expediting and the scheduler has released it's reference to the expediting thread before that was finished
		Expedited				= Expediting | ExpeditedFlag, 			//means the task was expedited
		Completed				= Running | CompletedFlag,				//means the task is completed with execution 
		ExpeditedAndCompleted	= Expedited | CompletedFlag,			//means the task is completed with execution and the runnable was expedited
		CanceledAndCompleted	= CanceledAndRunning | CompletedFlag,	//means the task is completed with execution of it's continuation but the runnable was cancelled
	};
	ENUM_CLASS_FLAGS(ETaskState)

	/**
	 * Generic implementation of a Deleter, it often comes up that one has to call a function to cleanup after a Task finished.
	 * This can be done by capturing a TDeleter like so: [Deleter(LowLevelTasks::TDeleter<Type, &Type::DeleteFunction>(value))](){}
	 */
	template<typename Type, void (Type::*DeleteFunction)()>
	class TDeleter
	{
		Type* Value;

	public:
		inline TDeleter(Type* InValue) : Value(InValue)
		{
		}

		inline TDeleter(const TDeleter&) = delete;
		inline TDeleter(TDeleter&& Other) : Value(Other.Value)
		{
			Other.Value = nullptr;
		}

		inline Type* operator->() const
		{
			return Value;
		}

		inline ~TDeleter() 
		{
			if(Value)
			{
				(Value->*DeleteFunction)();
			}
		}
	};

	class FTask;
	namespace Tasks_Impl
	{

	/**
	 * This class is just here to hide some variables away
	 * because we don't want to become too close friends with the FScheduler
	 */
	class FTaskBase
	{
		class FPackedDataAtomic;

		friend class ::LowLevelTasks::FTask;
		UE_NONCOPYABLE(FTaskBase); //means non movable

		union FPackedData
		{
			uintptr_t PackedData;
			struct
			{
				uintptr_t State				: 6;
				uintptr_t DebugName			: 53;
				uintptr_t Priority			: 3;
				uintptr_t Flags				: 2;
			};

		private:
			friend class FTaskBase::FPackedDataAtomic;
			FPackedData(uintptr_t InPackedData) : PackedData(InPackedData)
			{}

			constexpr FPackedData() 				
				: State((uintptr_t)ETaskState::CompletedFlag)
				, DebugName(0ull)
				, Priority((uintptr_t)ETaskPriority::Count)
				, Flags((uintptr_t)ETaskFlags::DefaultFlags)
			{
				static_assert(uintptr_t(ETaskPriority::Count) <= (1ull << 3), "Not enough bits to store ETaskPriority");
				static_assert(uintptr_t(ETaskState::Count) <= (1ull << 6), "Not enough bits to store ETaskState");
				static_assert(uintptr_t(ETaskFlags::AllowEverything) < (1ull << 2), "Not enough bits to store ETaskFlags");
			}

		public:
			FPackedData(const TCHAR* InDebugName, ETaskPriority InPriority, ETaskState InState, ETaskFlags InFlags)
				: State((uintptr_t)InState)
				, DebugName(reinterpret_cast<uintptr_t>(InDebugName))
				, Priority((uintptr_t)InPriority)
				, Flags((uintptr_t)InFlags)
				
			{
				checkSlow(reinterpret_cast<uintptr_t>(InDebugName) < (1ull << 53));
				checkSlow((uintptr_t)InPriority < (1ull << 3));
				checkSlow((uintptr_t)InState < (1ull << 6));
				checkSlow((uintptr_t)InFlags < (1ull << 2));
				static_assert(sizeof(FPackedData) == sizeof(uintptr_t), "Packed data needs to be pointer size");
			}

			FPackedData(const FPackedData& Other, ETaskState State)
				: FPackedData(Other.GetDebugName(), Other.GetPriority(), State, Other.GetFlags())
			{
			}

			inline const TCHAR* GetDebugName() const
			{
				return reinterpret_cast<const TCHAR*>(DebugName);
			}

			inline ETaskPriority GetPriority() const
			{
				return ETaskPriority(Priority);
			}

			inline ETaskState GetState() const
			{
				return ETaskState(State);
			}

			inline ETaskFlags GetFlags() const
			{
				return ETaskFlags(Flags);
			}
		};

		class FPackedDataAtomic
		{
			std::atomic<uintptr_t> PackedData { FPackedData().PackedData };

		public:
			ETaskState fetch_or(ETaskState State, std::memory_order Order)
			{
				return ETaskState(FPackedData(PackedData.fetch_or(uintptr_t(State), Order)).State);
			}

			bool compare_exchange_strong(FPackedData& Expected, FPackedData Desired, std::memory_order Success, std::memory_order Failure)
			{
				return PackedData.compare_exchange_strong(Expected.PackedData, Desired.PackedData, Success, Failure);
			}

			bool compare_exchange_strong(FPackedData& Expected, FPackedData Desired, std::memory_order Order)
			{
				return PackedData.compare_exchange_strong(Expected.PackedData, Desired.PackedData, Order);
			}

			FPackedData load(std::memory_order Order) const
			{
				return PackedData.load(Order);
			}

			void store(const FPackedData& Expected, std::memory_order Order)
			{
				PackedData.store(Expected.PackedData, Order);
			}
		};

	private:
		using FTaskDelegate = TTaskDelegate<FTask*(bool), LOWLEVEL_TASK_SIZE - sizeof(FPackedData) - sizeof(void*)>;
		FTaskDelegate Runnable;
		mutable void* UserData = nullptr;
		FPackedDataAtomic PackedData;

	private:
		FTaskBase() = default;
	};
	}


	/**
	 * Low level task implementation, this provides the state machine interface.
	 * The classes in UE::Tasks handle actual task execution.
	 */
	class FTask final : private Tasks_Impl::FTaskBase
	{		
		friend class FScheduler;
		UE_NONCOPYABLE(FTask); //means non movable

		static thread_local FTask* ActiveTask;

	public:

		/** If true, this means the task is completed and this taskhandle can be recycled */
		inline bool IsCompleted(std::memory_order MemoryOrder = std::memory_order_seq_cst) const
		{
			ETaskState State = PackedData.load(MemoryOrder).GetState();
			return EnumHasAnyFlags(State, ETaskState::CompletedFlag);
		}
		
		/** If true, this means the task was canceled but might still need to be launched */
		inline bool WasCanceled() const
		{
			ETaskState State = PackedData.load(std::memory_order_relaxed).GetState();
			return EnumHasAnyFlags(State, ETaskState::CanceledFlag);
		}

		/** If true, this means the task was expedited or that it already completed */
		inline bool WasExpedited() const
		{
			ETaskState State = PackedData.load(std::memory_order_acquire).GetState();
			return EnumHasAnyFlags(State, ETaskState::ExpeditedFlag | ETaskState::CompletedFlag);
		}

		/** If true, this means the task is ready to be launched but might have already been canceled */
		inline bool IsReady() const 
		{
			ETaskState State = PackedData.load(std::memory_order_relaxed).GetState();
			return !EnumHasAnyFlags(State, ~ETaskState::CanceledFlag); 
		}

#if PLATFORM_DESKTOP || !IS_MONOLITHIC
		/** Called by internal code to get the low level task executing on the current thread, may return null */
		CORE_API static const FTask* GetActiveTask();
#else
		UE_FORCEINLINE_HINT static const FTask* GetActiveTask()
		{
			return ActiveTask;
		}
#endif

		/** Try to cancel the task if it has not been launched yet */
		inline bool TryCancel(ECancellationFlags CancellationFlags = ECancellationFlags::DefaultFlags);

		/** 
		 * Try to revive a canceled task (as in reverting the cancellation as if it never happened).
		 * If it had been canceled and the scheduler has not run it yet this will succeed and return true.
		 */
		inline bool TryRevive();

		/**
		 * Try to expedite the task, if possible it will run immediately 
		 * but it will not set the completed state until the scheduler has executed it, because the scheduler still holds a reference.
		 * To check for completion in the context of expediting use WasExpedited. The TaskHandle cannot be reused until IsCompleted returns true.
		 * @param Continuation: optional Continuation that needs to be executed or scheduled by the caller (can only be non null if the operation returned true)
		 */
		inline bool TryExpedite();
		inline bool TryExpedite(FTask*& Continuation);

		/**
		 * Try to execute the task, if it has not been launched yet the task will execute immediately.
		 * @param Continuation: optional Continuation that needs to be executed or scheduled by the caller (can only be non null if the operation returned true)
		 */
		inline bool TryExecute();
		inline bool TryExecute(FTask*& Continuation);

		/** Initialize a new task and prepare for execution */
		template<typename TRunnable>
		inline void Init(const TCHAR* InDebugName, ETaskPriority InPriority, TRunnable&& InRunnable, ETaskFlags Flags = ETaskFlags::DefaultFlags);

		template<typename TRunnable>
		inline void Init(const TCHAR* InDebugName, TRunnable&& InRunnable, ETaskFlags Flags = ETaskFlags::DefaultFlags);

		inline const TCHAR* GetDebugName() const;
		inline ETaskPriority GetPriority() const;
		inline bool IsBackgroundTask() const;
		inline bool AllowBusyWaiting() const;
		inline bool AllowCancellation() const;

		struct FInitData
		{
			const TCHAR* DebugName;
			ETaskPriority Priority;
			ETaskFlags Flags;
		};
		inline FInitData GetInitData() const;

		void* GetUserData() const { return UserData; }
		void SetUserData(void* NewUserData) const { UserData = NewUserData; }

		FTask() = default;
		inline ~FTask();

	private: //Interface of the Scheduler
		inline static bool PermitBackgroundWork()
		{
			return ActiveTask && ActiveTask->IsBackgroundTask();
		}

		inline bool WasCanceledOrIsExpediting() const
		{
			ETaskState State = PackedData.load(std::memory_order_relaxed).GetState();
			return EnumHasAnyFlags(State, ETaskState::CanceledFlag | ETaskState::RunningFlag);
		}

		inline bool TryPrepareLaunch();
		//after calling this function the task can be considered dead
		template<bool bIsExpeditingThread>
		inline void TryFinish();

		inline FTask* ExecuteTask();
		inline void InheritParentData(ETaskPriority& Priority);
	};

   /******************
	* IMPLEMENTATION *
	******************/

	inline ETaskPriority FTask::GetPriority() const 
	{ 
		return PackedData.load(std::memory_order_relaxed).GetPriority(); 
	}

	inline void FTask::InheritParentData(ETaskPriority& Priority)
	{
		const FTask* LocalActiveTask = FTask::GetActiveTask();
		if (LocalActiveTask != nullptr)
		{
			if (Priority == ETaskPriority::Inherit)
			{
				Priority = LocalActiveTask->GetPriority();
			}
			UserData = LocalActiveTask->GetUserData();
		}
		else
		{
			if (Priority == ETaskPriority::Inherit)
			{
				Priority = ETaskPriority::Default;
			}
			UserData = nullptr;
		}
	}

	template<typename TRunnable>
	inline void FTask::Init(const TCHAR* InDebugName, ETaskPriority InPriority, TRunnable&& InRunnable, ETaskFlags Flags)
	{
		checkf(IsCompleted(), TEXT("State: %d"), int32{EnumToUnderlyingType(PackedData.load(std::memory_order_relaxed).GetState())});
		checkSlow(!Runnable.IsSet());
		
		//if the Runnable returns an FTask* than enable symetric switching
		if constexpr (std::is_same_v<FTask*, decltype(UE::Core::Private::IsInvocable::DeclVal<TRunnable>()())>)
		{
			Runnable = [LocalRunnable = Forward<TRunnable>(InRunnable)](const bool bNotCanceled) mutable -> FTask*
			{
				if (bNotCanceled)
				{
					FTask* Task = LocalRunnable();
					return Task;
				}
				return nullptr;
			};
		}
		else
		{
			Runnable = [LocalRunnable = Forward<TRunnable>(InRunnable)](const bool bNotCanceled) mutable -> FTask* 
			{
				if (bNotCanceled)
				{
					LocalRunnable();
				}
				return nullptr;
			};
		}
		InheritParentData(InPriority);
		PackedData.store(FPackedData(InDebugName, InPriority, ETaskState::Ready, Flags), std::memory_order_release);
	}

	template<typename TRunnable>
	inline void FTask::Init(const TCHAR* InDebugName, TRunnable&& InRunnable, ETaskFlags Flags)
	{
		Init(InDebugName, ETaskPriority::Default, Forward<TRunnable>(InRunnable), Flags);
	}

	inline FTask::~FTask()
	{
		checkf(IsCompleted(), TEXT("State: %d"), int32{EnumToUnderlyingType(PackedData.load(std::memory_order_relaxed).GetState())});
	}

	inline bool FTask::TryPrepareLaunch()
	{
		return !EnumHasAnyFlags(PackedData.fetch_or(ETaskState::ScheduledFlag, std::memory_order_release), ETaskState::ScheduledFlag);
	}

	inline bool FTask::TryCancel(ECancellationFlags CancellationFlags)
	{
		bool bPrelaunchCancellation = EnumHasAnyFlags(CancellationFlags, ECancellationFlags::PrelaunchCancellation);
		bool bTryLaunchOnSuccess	= EnumHasAllFlags(CancellationFlags, ECancellationFlags::PrelaunchCancellation | ECancellationFlags::TryLaunchOnSuccess);

		FPackedData LocalPackedData = PackedData.load(std::memory_order_relaxed);
		FPackedData ReadyState(LocalPackedData, ETaskState::Ready);
		FPackedData ScheduledState(LocalPackedData, ETaskState::Scheduled);
		//to launch a canceled  task it has to go though TryPrepareLaunch which is doing the memory_order_release
		bool WasCanceled = EnumHasAnyFlags(LocalPackedData.GetFlags(), ETaskFlags::AllowCancellation)
			&& ((bPrelaunchCancellation && PackedData.compare_exchange_strong(ReadyState, FPackedData(LocalPackedData, ETaskState::CanceledAndReady), std::memory_order_acquire))
									    || PackedData.compare_exchange_strong(ScheduledState, FPackedData(LocalPackedData, ETaskState::Canceled), std::memory_order_acquire));

		if(bTryLaunchOnSuccess && WasCanceled && TryPrepareLaunch())
		{
			verifySlow(ExecuteTask() == nullptr);
			return true;
		}
		return WasCanceled;
	}

	inline bool FTask::TryRevive()
	{
		FPackedData LocalPackedData = PackedData.load(std::memory_order_relaxed);
		checkSlow(EnumHasAnyFlags(LocalPackedData.GetState(), ETaskState::CanceledFlag));
		if(EnumHasAnyFlags(LocalPackedData.GetState(), ETaskState::RunningFlag))
		{
			return false;
		}

		FPackedData CanceledReadyState(LocalPackedData, ETaskState::CanceledAndReady);
		FPackedData CanceledState(LocalPackedData, ETaskState::Canceled);
		return PackedData.compare_exchange_strong(CanceledReadyState, FPackedData(LocalPackedData, ETaskState::Ready), std::memory_order_release)
			|| PackedData.compare_exchange_strong(CanceledState, FPackedData(LocalPackedData, ETaskState::Scheduled), std::memory_order_release);
	}

	inline bool FTask::TryExecute(FTask*& OutContinuation)
	{
		if(TryPrepareLaunch())
		{
			OutContinuation  = ExecuteTask();
			return true;
		}
		return false;
	}

	inline bool FTask::TryExecute()
	{
		FTask* Continuation = nullptr;
		bool Result = TryExecute(Continuation);
		checkSlow(Continuation == nullptr);
		return Result;
	}

	template<bool bIsExpeditingThread>
	inline void FTask::TryFinish()
	{
		const ETaskState NextState = bIsExpeditingThread ? ETaskState::ExpeditedFlag | ETaskState::ExpeditingFlag : ETaskState::ExpeditingFlag;
		ETaskState PreviousState = PackedData.fetch_or(NextState, std::memory_order_acq_rel);
		if constexpr (bIsExpeditingThread)
		{
			checkSlow(PreviousState == ETaskState::Running || PreviousState == ETaskState::Expediting);
		}
		if(EnumHasAnyFlags(PreviousState, ETaskState::ExpeditingFlag))
		{
			FTaskDelegate LocalRunnable = MoveTemp(Runnable);
			//do not access the task again after this call
			//as by defitition the task can be considered dead
			PreviousState = PackedData.fetch_or(ETaskState::CompletedFlag, std::memory_order_seq_cst);
			checkSlow(PreviousState == ETaskState::Expedited);
		}
	}

	inline bool FTask::TryExpedite(FTask*& OutContinuation)
	{
		FPackedData LocalPackedData = PackedData.load(std::memory_order_relaxed);
		FPackedData ScheduledState(LocalPackedData, ETaskState::Scheduled);
		if(PackedData.compare_exchange_strong(ScheduledState, FPackedData(LocalPackedData, ETaskState::Running), std::memory_order_acquire))
		{
			OutContinuation  = Runnable(true);
			TryFinish<true>();
			return true;
		}
		return false;
	}

	inline bool FTask::TryExpedite()
	{
		FTask* Continuation = nullptr;
		bool Result = TryExpedite(Continuation);
		checkSlow(Continuation == nullptr);
		return Result;
	}

	inline FTask* FTask::ExecuteTask()
	{
		ETaskState PreviousState = PackedData.fetch_or(ETaskState::RunningFlag, std::memory_order_acquire);
		checkSlow(EnumHasAnyFlags(PreviousState, ETaskState::ScheduledFlag));

		FTask* Continuation = nullptr;
		if(!EnumHasAnyFlags(PreviousState, ETaskState::RunningFlag)) //we are running or canceled
		{
			FTaskDelegate LocalRunnable;
			Continuation = Runnable.CallAndMove(LocalRunnable, !EnumHasAnyFlags(PreviousState, ETaskState::CanceledFlag));
			//do not access the task again after this call
			//as by defitition the task can be considered dead
			PreviousState = PackedData.fetch_or(ETaskState::CompletedFlag, std::memory_order_seq_cst);
			checkSlow(PreviousState == ETaskState::Running || PreviousState == ETaskState::CanceledAndRunning);
		}
		else // we are expedited
		{
			checkSlow(PreviousState == ETaskState::Running || PreviousState == ETaskState::Expediting || PreviousState == ETaskState::Expedited);
			TryFinish<false>();
		}

		return Continuation;
	}

	inline const TCHAR* FTask::GetDebugName() const
	{
		return PackedData.load(std::memory_order_relaxed).GetDebugName(); 
	}

	inline bool FTask::IsBackgroundTask() const
	{
		return PackedData.load(std::memory_order_relaxed).GetPriority() >= ETaskPriority::ForegroundCount;
	}

	inline bool FTask::AllowBusyWaiting() const
	{
		return EnumHasAnyFlags(PackedData.load(std::memory_order_relaxed).GetFlags(), ETaskFlags::AllowBusyWaiting);
	}

	inline bool FTask::AllowCancellation() const
	{
		return EnumHasAnyFlags(PackedData.load(std::memory_order_relaxed).GetFlags(), ETaskFlags::AllowCancellation);
	}

	inline FTask::FInitData FTask::GetInitData() const
	{
		FPackedData LocalPackedData = PackedData.load(std::memory_order_relaxed);
		return { LocalPackedData.GetDebugName(), LocalPackedData.GetPriority(), LocalPackedData.GetFlags() };
	}
}
