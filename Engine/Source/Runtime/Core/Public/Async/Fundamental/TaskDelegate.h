// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Experimental/ConcurrentLinearAllocator.h"
#include "Templates/IsInvocable.h"
#include "Misc/ScopeExit.h"
#include "Misc/Launder.h"
#include <type_traits>

namespace LowLevelTasks
{
	struct FLowLevelTasksBlockAllocationTag : FDefaultBlockAllocationTag
	{
		static constexpr uint32 BlockSize = 64 * 1024;
		static constexpr bool AllowOversizedBlocks = true;
		static constexpr bool RequiresAccurateSize = false;
		static constexpr bool InlineBlockAllocation = true;
		static constexpr const char* TagName = "LowLevelTasksLinear";

		using Allocator = TBlockAllocationCache<BlockSize, FAlignedAllocator>;
	};

	namespace TTaskDelegate_Impl
	{
		template<typename ReturnType>
		inline ReturnType MakeDummyValue()
		{
			return *(reinterpret_cast<ReturnType*>(uintptr_t(1)));
		}

		template<>
		inline void MakeDummyValue<void>()
		{
			return;
		}
	}
	//version of TUniqueFunction<ReturnType()> that is less wasteful with it's memory
	//this class might be removed when TUniqueFunction<ReturnType()> is fixed
	template<typename = void(), uint32 = PLATFORM_CACHE_LINE_SIZE>
	class TTaskDelegate;

	template<uint32 TotalSize , typename ReturnType, typename... ParamTypes>
	class alignas(8) TTaskDelegate<ReturnType(ParamTypes...), TotalSize>
	{
		template<typename, uint32>
		friend class TTaskDelegate;

		using ThisClass = TTaskDelegate<ReturnType(ParamTypes...), TotalSize>;
		struct TTaskDelegateBase
		{
			virtual void Move(TTaskDelegateBase&, void*, void*, uint32) 
			{
				checkNoEntry(); //if we get to this place than the compiler is optimizing our vtable lookup and ignores our undefined/shallow copy of the memory
			};

			virtual ReturnType Call(void*, ParamTypes...) const 
			{ 
				checkNoEntry(); //if we get to this place than the compiler is optimizing our vtable lookup and ignores our undefined/shallow copy of the memory 
				return TTaskDelegate_Impl::MakeDummyValue<ReturnType>(); 
			};

			virtual ReturnType CallAndMove(ThisClass&, void*, uint32, ParamTypes...) 
			{
				checkNoEntry(); //if we get to this place than the compiler is optimizing our vtable lookup and ignores our undefined/shallow copy of the memory 
				return TTaskDelegate_Impl::MakeDummyValue<ReturnType>(); 
			};

			virtual void Destroy(void*) 
			{ 
				checkNoEntry(); //if we get to this place than the compiler is optimizing our vtable lookup and ignores our undefined/shallow copy of the memory 
			};

			virtual bool IsHeapAllocated() const 
			{ 
				checkNoEntry(); //if we get to this place than the compiler is optimizing our vtable lookup and ignores our undefined/shallow copy of the memory
				return false; 
			}

			virtual bool IsSet() const 
			{ 
				checkNoEntry(); //if we get to this place than the compiler is optimizing our vtable lookup and ignores our undefined/shallow copy of the memory 
				return false; 
			}

			virtual uint32 DelegateSize() const 
			{ 
				checkNoEntry(); //if we get to this place than the compiler is optimizing our vtable lookup and ignores our undefined/shallow copy of the memory 
				return 0; 
			}
		};

		struct TTaskDelegateDummy final : TTaskDelegateBase
		{
			void Move(TTaskDelegateBase&, void*, void*, uint32) override 
			{
			}

			ReturnType Call(void*, ParamTypes...) const override 
			{ 
				checkf(false, TEXT("trying to Call a dummy TaskDelegate"));
				return TTaskDelegate_Impl::MakeDummyValue<ReturnType>(); 
			}

			ReturnType CallAndMove(ThisClass&, void*, uint32, ParamTypes...) override 
			{ 
				checkf(false, TEXT("trying to Call a dummy TaskDelegate"));
				return TTaskDelegate_Impl::MakeDummyValue<ReturnType>(); 
			}

			void Destroy(void*) override 
			{
			}

			bool IsHeapAllocated() const override 
			{ 
				return false; 
			}

			bool IsSet() const override 
			{ 
				return false; 
			}

			uint32 DelegateSize() const override 
			{ 
				return 0; 
			}
		};

		template<typename TCallableType, bool HeapAllocated>
		struct TTaskDelegateImpl;

		template<typename TCallableType>
		struct TTaskDelegateImpl<TCallableType, false> final : TTaskDelegateBase
		{
			template<typename CallableT>
			inline TTaskDelegateImpl(CallableT&& Callable, void* InlineData)
			{
				static_assert(::TIsInvocable<TCallableType, ParamTypes...>::Value, "TCallableType is not invocable");
				static_assert(std::is_same_v<ReturnType, decltype(Callable(UE::Core::Private::IsInvocable::DeclVal<ParamTypes>()...))>, "TCallableType return type does not match");
				static_assert(sizeof(TTaskDelegateImpl<TCallableType, false>) == sizeof(TTaskDelegateBase), "Size must match the Baseclass");
				new (InlineData) TCallableType(Forward<CallableT>(Callable));
			}

			inline void Move(TTaskDelegateBase& DstWrapper, void* DstData, void* SrcData, uint32 DestInlineSize) override
			{
				TCallableType* SrcPtr = reinterpret_cast<TCallableType*>(SrcData);
				if ((sizeof(TCallableType) <= DestInlineSize) && (uintptr_t(DstData) % alignof(TCallableType)) == 0)
				{
					new (&DstWrapper) TTaskDelegateImpl<TCallableType, false>(MoveTemp(*SrcPtr), DstData);
				}
				else
				{
					new (&DstWrapper) TTaskDelegateImpl<TCallableType, true>(MoveTemp(*SrcPtr), DstData);
				}
				new (this) TTaskDelegateDummy();
			}

			inline ReturnType Call(void* InlineData, ParamTypes... Params) const override
			{
				TCallableType* LocalPtr = reinterpret_cast<TCallableType*>(InlineData);
				return Invoke(*LocalPtr, Params...);
			}

			ReturnType CallAndMove(ThisClass& Destination, void* InlineData, uint32 DestInlineSize, ParamTypes... Params) override
			{
				ON_SCOPE_EXIT
				{
					Move(Destination.CallableWrapper, Destination.InlineStorage, InlineData, DestInlineSize);
				};
				return Call(InlineData, Params...);		
			}

			void Destroy(void* InlineData) override
			{
				TCallableType* LocalPtr = reinterpret_cast<TCallableType*>(InlineData);
				LocalPtr->~TCallableType();
			}

			bool IsHeapAllocated() const override
			{ 
				return false; 
			}

			bool IsSet() const override 
			{ 
				return true; 
			}

			uint32 DelegateSize() const override 
			{ 
				return sizeof(TCallableType); 
			}
		};

		template<typename TCallableType>
		struct TTaskDelegateImpl<TCallableType, true> final : TTaskDelegateBase
		{
		private:
			inline TTaskDelegateImpl(void* DstData, void* SrcData)
			{
				memcpy(DstData, SrcData, sizeof(TCallableType*));
			}

		public:
			template<typename CallableT>
			inline TTaskDelegateImpl(CallableT&& Callable, void* InlineData)
			{
				static_assert(::TIsInvocable<TCallableType, ParamTypes...>::Value, "TCallableType is not invocable");
				static_assert(std::is_same_v<ReturnType, decltype(Callable(UE::Core::Private::IsInvocable::DeclVal<ParamTypes>()...))>, "TCallableType return type does not match");
				static_assert(sizeof(TTaskDelegateImpl<TCallableType, true>) == sizeof(TTaskDelegateBase), "Size must match the Baseclass");
				static_assert(alignof(TCallableType) <= FLowLevelTasksBlockAllocationTag::Allocator::MaxAlignment);
				TCallableType** HeapPtr = reinterpret_cast<TCallableType**>(InlineData);
				*HeapPtr = reinterpret_cast<TCallableType*>(TConcurrentLinearAllocator<FLowLevelTasksBlockAllocationTag>::template Malloc<alignof(TCallableType)>(sizeof(TCallableType)));
				new (*HeapPtr) TCallableType(Forward<CallableT>(Callable));
			}

			inline void Move(TTaskDelegateBase& DstWrapper, void* DstData, void* SrcData, uint32 DestInlineSize) override
			{
				new (&DstWrapper) TTaskDelegateImpl<TCallableType, true>(DstData, SrcData);
				new (this) TTaskDelegateDummy();
			}

			inline ReturnType Call(void* InlineData, ParamTypes... Params) const override
			{
				TCallableType* HeapPtr = reinterpret_cast<TCallableType*>(*reinterpret_cast<void* const*>(InlineData));
				return Invoke(*HeapPtr, Params...);
			}

			ReturnType CallAndMove(ThisClass& Destination, void* InlineData, uint32 DestInlineSize, ParamTypes... Params) override
			{
				ON_SCOPE_EXIT
				{
					Move(Destination.CallableWrapper, Destination.InlineStorage, InlineData, DestInlineSize);
				};
				return Call(InlineData, Params...);			
			}

			void Destroy(void* InlineData) override
			{
				TCallableType* HeapPtr = reinterpret_cast<TCallableType*>(*reinterpret_cast<void**>(InlineData));
				// We need a typedef here because VC won't compile the destructor call below if ElementType itself has a member called ElementType
				using DestructorType = TCallableType;
				HeapPtr->DestructorType::~TCallableType();
				TConcurrentLinearAllocator<FLowLevelTasksBlockAllocationTag>::Free(HeapPtr);
			}

			bool IsHeapAllocated() const override 
			{ 
				return true; 
			}

			bool IsSet() const override 
			{ 
				return true; 
			}

			uint32 DelegateSize() const override 
			{ 
				return sizeof(TCallableType); 
			}
		};

	public:
		TTaskDelegate() 
		{
			static_assert(TotalSize % 8 == 0,  "Totalsize must be dividable by 8");
			static_assert(TotalSize >= (sizeof(TTaskDelegateBase) + sizeof(void*)),  "Totalsize must be large enough to fit a vtable and pointer");
			new (&CallableWrapper) TTaskDelegateDummy();
		}

		template<uint32 SourceTotalSize>
		TTaskDelegate(const TTaskDelegate<ReturnType(ParamTypes...), SourceTotalSize>&) = delete;

		template<uint32 SourceTotalSize>
		TTaskDelegate(TTaskDelegate<ReturnType(ParamTypes...), SourceTotalSize>&& Other)
		{
			Other.GetWrapper()->Move(CallableWrapper, InlineStorage, Other.InlineStorage, InlineStorageSize);
		}

		template<typename CallableT>
		TTaskDelegate(CallableT&& Callable)
		{
			using TCallableType = std::decay_t<CallableT>;
			if constexpr ((sizeof(TCallableType) <= InlineStorageSize) && ((uintptr_t(InlineStorageSize) % alignof(TCallableType)) == 0))
			{
				new (&CallableWrapper) TTaskDelegateImpl<TCallableType, false>(Forward<CallableT>(Callable), InlineStorage);
			}
			else
			{
				new (&CallableWrapper) TTaskDelegateImpl<TCallableType, true>(Forward<CallableT>(Callable), InlineStorage);
			}
		}

		~TTaskDelegate()
		{
			GetWrapper()->Destroy(InlineStorage);
		}

		ReturnType operator()(ParamTypes... Params) const
		{
			return GetWrapper()->Call(InlineStorage, Params...);
		}

		template<uint32 DestTotalSize>
		ReturnType CallAndMove(TTaskDelegate<ReturnType(ParamTypes...), DestTotalSize>& Destination, ParamTypes... Params)
		{
			checkSlow(!Destination.IsSet());
			return GetWrapper()->CallAndMove(Destination, InlineStorage, TTaskDelegate<ReturnType(ParamTypes...), DestTotalSize>::InlineStorageSize, Params...);
		}

		template<uint32 SourceTotalSize>
		ThisClass& operator= (const TTaskDelegate<ReturnType(ParamTypes...), SourceTotalSize>&) = delete;

		template<uint32 SourceTotalSize>
		ThisClass& operator= (TTaskDelegate<ReturnType(ParamTypes...), SourceTotalSize>&& Other)
		{
			GetWrapper()->Destroy(InlineStorage);
			Other.GetWrapper()->Move(CallableWrapper, InlineStorage, Other.InlineStorage, InlineStorageSize);
			return *this;
		}

		template<typename CallableT>
		ThisClass& operator= (CallableT&& Callable)
		{
			using TCallableType = std::decay_t<CallableT>;
			GetWrapper()->Destroy(InlineStorage);
			if constexpr ((sizeof(TCallableType) <= InlineStorageSize) && ((uintptr_t(InlineStorageSize) % alignof(TCallableType)) == 0))
			{
				new (&CallableWrapper) TTaskDelegateImpl<TCallableType, false>(Forward<CallableT>(Callable), InlineStorage);
			}
			else
			{
				new (&CallableWrapper) TTaskDelegateImpl<TCallableType, true>(Forward<CallableT>(Callable), InlineStorage);
			}
			return *this;
		}

		void Destroy()
		{
			GetWrapper()->Destroy(InlineStorage);
			new (&CallableWrapper) TTaskDelegateDummy();
		}

		bool IsHeapAllocated() const
		{
			return GetWrapper()->IsHeapAllocated();
		}

		bool IsSet() const
		{
			return GetWrapper()->IsSet();
		}

		uint32 DelegateSize() const  
		{ 
			return GetWrapper()->DelegateSize(); 
		}

	private:
		static constexpr uint32 InlineStorageSize = TotalSize - sizeof(TTaskDelegateBase);
		mutable char InlineStorage[InlineStorageSize];
		TTaskDelegateBase CallableWrapper;

		TTaskDelegateBase* GetWrapper()
		{
			return UE_LAUNDER(static_cast<TTaskDelegateBase*>(&CallableWrapper));
		}

		const TTaskDelegateBase* GetWrapper() const
		{
			return UE_LAUNDER(static_cast<const TTaskDelegateBase*>(&CallableWrapper));
		}
	};
}
