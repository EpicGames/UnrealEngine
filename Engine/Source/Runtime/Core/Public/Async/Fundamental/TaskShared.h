// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Delegates/Delegate.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

namespace LowLevelTasks {

DECLARE_TS_MULTICAST_DELEGATE(FOversubscriptionLimitReached);

template<typename NodeType>
using TAlignedArray = TArray<NodeType, TAlignedHeapAllocator<alignof(NodeType)>>;

namespace Private {

class FOutOfWork
{
private:
	bool ActivelyLookingForWork = false;

#if CPUPROFILERTRACE_ENABLED
	bool bCpuBeginEventEmitted = false;
#endif
public:
	inline ~FOutOfWork()
	{
		Stop();
	}

	inline bool Start()
	{
		if (!ActivelyLookingForWork)
		{
#if CPUPROFILERTRACE_ENABLED
			if (CpuChannel)
			{
				static uint32 WorkerLookingForWorkTraceId = FCpuProfilerTrace::OutputEventType("TaskWorkerIsLookingForWork");
				FCpuProfilerTrace::OutputBeginEvent(WorkerLookingForWorkTraceId);
				bCpuBeginEventEmitted = true;
			}
#endif
			ActivelyLookingForWork = true;
			return true;
		}
		return false;
	}

	inline bool Stop()
	{
		if (ActivelyLookingForWork)
		{
#if CPUPROFILERTRACE_ENABLED
			if (bCpuBeginEventEmitted)
			{
				FCpuProfilerTrace::OutputEndEvent();
				bCpuBeginEventEmitted = false;
			}
#endif
			ActivelyLookingForWork = false;
			return true;
		}
		return false;
	}
};

} // namespace Private

} // namespace LowLevelTasks