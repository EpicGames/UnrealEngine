// Copyright (C) 2016 Dmitry Vyukov <dvyukov@google.com>
//
// This Source Code Form is subject to the terms of the Mozilla
// Public License v. 2.0. If a copy of the MPL was not distributed
// with this file, You can obtain one at http://mozilla.org/MPL/2.0/.

// This implementation is based on EventCount.h
// included in the Eigen library but almost everything has been
// rewritten.

#pragma once 

#include "Async/Fundamental/TaskShared.h"
#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "HAL/Event.h"

#include <atomic>

namespace LowLevelTasks::Private
{
	enum class EWaitState
	{
		NotSignaled = 0,
		Waiting,
		Signaled,
	};

	/*
	* the struct is naturally 64 bytes aligned, the extra alignment just
	* re-enforces this assumption and will error if it changes in the future
	*/
	struct alignas(64) FWaitEvent
	{
		std::atomic<uint64>     Next{ 0 };
		uint64                  Epoch{ 0 };
		std::atomic<EWaitState> State{ EWaitState::NotSignaled };
		FEventRef               Event{ EEventMode::ManualReset };
		bool                    bIsStandby{ false };
	};

	class FWaitingQueue
	{
		uint32                         ThreadCount{ 0 };    // Normal amount of threads when there is no oversubscription.
		uint32                         MaxThreadCount{ 0 }; // Max limit that can be reached during oversubscription period.
		TFunction<void()>              CreateThread;
		std::atomic<uint32>            Oversubscription{ 0 };
		std::atomic<uint64>            State;
		std::atomic<uint64>            StandbyState;
		TAlignedArray<FWaitEvent>&     NodesArray;
		std::atomic<bool>              bIsShuttingDown{ false };
		FOversubscriptionLimitReached& OversubscriptionLimitReachedEvent;
	public:
		FWaitingQueue(TAlignedArray<FWaitEvent>& InNodesArray, FOversubscriptionLimitReached& InOversubscriptionLimitReachedEvent)
			: NodesArray(InNodesArray)
			, OversubscriptionLimitReachedEvent(InOversubscriptionLimitReachedEvent)
		{
		}

		CORE_API void Init(uint32 InThreadCount, uint32 InMaxThreadCount, TFunction<void()> InCreateThread, uint32 InActiveThreadCount);
		CORE_API void StartShutdown();
		CORE_API void FinishShutdown();

		// First step to execute when no more work is found in the queues.
		CORE_API void PrepareStandby(FWaitEvent* Node);
		// Second step to execute when no more work is found in the queues.
		CORE_API bool CommitStandby(FWaitEvent* Node, FOutOfWork& OutOfWork);

		// Immediately goes to sleep if oversubscription period is finished and we're over the allowed thread count.
		CORE_API void ConditionalStandby(FWaitEvent* Node);

		// First step run by normal workers when no more work is found in the queues.
		CORE_API void PrepareWait(FWaitEvent* Node);
		// Second step run by normal workers when no more work is found in the queues.
		CORE_API bool CommitWait(FWaitEvent* Node, FOutOfWork& OutOfWork, int32 SpinCycles, int32 WaitCycles);

		// Step to run by normal workers if they detect new work after they called prepare wait.
		// Returns true if we need to wake up a new worker.
		CORE_API bool CancelWait(FWaitEvent* Node);

		// Increment oversubscription and notify a thread if we're under the allowed thread count.
		// If dynamic thread creation is allowed, this could spawn a new thread if needed.
		CORE_API void IncrementOversubscription();

		// Decrement oversubscription only, any active threads will finish their current task and will
		// go to sleep if conditional standby determines we're now over the active thread count.
		CORE_API void DecrementOversubscription();

		// Is the current waiting queue out of workers
		CORE_API bool IsOversubscriptionLimitReached() const;

		// Try to wake up the amount of workers passed in the parameters.
		// Return the number that were woken up.
		int32 Notify(int32 Count = 1)
		{
			return NotifyInternal(Count);
		}

	private:
		CORE_API bool  TryStartNewThread();
		CORE_API int32 NotifyInternal(int32 Count);
		CORE_API void  Park(FWaitEvent* Node, FOutOfWork& OutOfWork, int32 SpinCycles, int32 WaitCycles);
		CORE_API int32 Unpark(FWaitEvent* InNode);
		CORE_API void  CheckState(uint64 State, bool bIsWaiter = false);
		CORE_API void  CheckStandbyState(uint64 State);
	};

} // namespace LowLevelTasks::Private