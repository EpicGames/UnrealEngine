// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/Function.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"
#include "Misc/DateTime.h"
#include "HAL/Event.h"
#include "HAL/PooledSyncEvent.h"
#include "Misc/ScopeLock.h"
#include <type_traits>

template<typename ResultType> class TFuture;
template<typename ResultType> class TSharedFuture;
template<typename ResultType> class TPromise;

/**
 * Base class for the internal state of asynchronous return values (futures).
 */
class FFutureState
{
public:

	/** Default constructor. */
	FFutureState() = default;

	/**
	 * Create and initialize a new instance with a callback.
	 *
	 * @param InCompletionCallback A function that is called when the state is completed.
	 */
	FFutureState(TUniqueFunction<void()>&& InCompletionCallback)
		: CompletionCallback(MoveTemp(InCompletionCallback))
	{
	}

public:

	/**
	 * Checks whether the asynchronous result has been set.
	 *
	 * @return true if the result has been set, false otherwise.
	 * @see WaitFor
	 */
	bool IsComplete() const
	{
		return bComplete;
	}

	/**
	 * Blocks the calling thread until the future result is available.
	 *
	 * @param Duration The maximum time span to wait for the future result.
	 * @return true if the result is available, false otherwise.
	 * @see IsComplete
	 */
	bool WaitFor(const FTimespan& Duration) const
	{
		if (CompletionEvent->Wait(Duration))
		{
			return true;
		}

		return false;
	}

	/** 
	 * Set a continuation to be called on completion of the promise
	 * @param Continuation 
	 */
	void SetContinuation(TUniqueFunction<void()>&& Continuation)
	{
		bool bShouldJustRun = IsComplete();
		if (!bShouldJustRun)
		{
			FScopeLock Lock(&Mutex);
			bShouldJustRun = IsComplete();
			if (!bShouldJustRun)
			{
				CompletionCallback = MoveTemp(Continuation);
			}
		}
		if (bShouldJustRun && Continuation)
		{
			Continuation();
		}
	}

protected:

	/** Notifies any waiting threads that the result is available. */
	void MarkComplete()
	{
		TUniqueFunction<void()> Continuation;
		{
			FScopeLock Lock(&Mutex);
			Continuation = MoveTemp(CompletionCallback);
			bComplete = true;
		}
		CompletionEvent->Trigger();

		if (Continuation)
		{
			Continuation();
		}
	}

private:
	/** Mutex used to allow proper handling of continuations */
	mutable FCriticalSection Mutex;

	/** An optional callback function that is executed the state is completed. */
	TUniqueFunction<void()> CompletionCallback;

	/** Holds an event signaling that the result is available. */
	FPooledSyncEvent CompletionEvent{ true };

	/** Whether the asynchronous result is available. */
	TAtomic<bool> bComplete{ false };
};


/**
 * Implements the internal state of asynchronous return values (futures).
 */
template<typename ResultType>
class TFutureState
	: public FFutureState
{
public:
	using MutableResultType = typename TTypeCompatibleBytes<ResultType>::MutableGetType;
	using ConstResultType   = typename TTypeCompatibleBytes<ResultType>::ConstGetType;
	using RvalueResultType  = typename TTypeCompatibleBytes<ResultType>::RvalueGetType;

	/** Default constructor. */
	TFutureState()
		: FFutureState()
	{ }

	~TFutureState()
	{
		if (IsComplete())
		{
			Result.DestroyUnchecked();
		}
	}

	/**
	 * Create and initialize a new instance with a callback.
	 *
	 * @param CompletionCallback A function that is called when the state is completed.
	 */
	TFutureState(TUniqueFunction<void()>&& CompletionCallback)
		: FFutureState(MoveTemp(CompletionCallback))
	{ }

public:

	/**
	 * Gets the result (will block the calling thread until the result is available).
	 *
	 * @return The result value.
	 * @see EmplaceResult
	 */
	MutableResultType GetResult()
	{
		while (!IsComplete())
		{
			WaitFor(FTimespan::MaxValue());
		}

		return Result.GetUnchecked();
	}
	ConstResultType GetResult() const
	{
		return const_cast<TFutureState*>(this)->GetResult();
	}

	/**
	 * Sets the result and notifies any waiting threads.
	 *
	 * @param Args The arguments to forward to the constructor of the result.
	 * @see GetResult
	 */
	template<typename... ArgTypes>
	void EmplaceResult(ArgTypes&&... Args)
	{
		check(!IsComplete());
		Result.EmplaceUnchecked(Forward<ArgTypes>(Args)...);
		MarkComplete();
	}

private:

	/** Holds the asynchronous result. */
	TTypeCompatibleBytes<ResultType> Result;
};

/* TFuture
*****************************************************************************/

/**
 * Abstract base template for futures and shared futures.
 */
template <typename ResultType>
class TFutureBase
{
protected:
	using StateType = TFutureState<ResultType>;

public:
	using MutableResultType = typename StateType::MutableResultType;
	using ConstResultType   = typename StateType::ConstResultType;
	using RvalueResultType  = typename StateType::RvalueResultType;

	/**
	 * Gets the future's result.
	 *
	 * @return The result as a const reference, or the same reference if the future holds a reference, or void if the future holds a void.
	 * @note Not equivalent to std::future::get(). The future remains valid.
	 */
	ConstResultType Get() const
		requires (!std::is_object_v<ResultType>)
	{
		return this->GetState()->GetResult();
	}
	ConstResultType Get() const UE_LIFETIMEBOUND
		requires (std::is_object_v<ResultType>)
	{
		return this->GetState()->GetResult();
	}

	/**
	 * Checks whether this future object has its value set.
	 *
	 * @return true if this future has a shared state and the value has been set, false otherwise.
	 * @see IsValid
	 */
	bool IsReady() const
	{
		return State.IsValid() ? State->IsComplete() : false;
	}

	/**
	 * Checks whether this future object has a valid state.
	 *
	 * @return true if the state is valid, false otherwise.
	 * @see IsReady
	 */
	bool IsValid() const
	{
		return State.IsValid();
	}

	/**
	 * Blocks the calling thread until the future result is available.
	 *
	 * Note that this method may block forever if the result is never set. Use
	 * the WaitFor or WaitUntil methods to specify a maximum timeout for the wait.
	 *
	 * @see WaitFor, WaitUntil
	 */
	void Wait() const
	{
		if (State.IsValid())
		{
			while (!WaitFor(FTimespan::MaxValue()));
		}
	}

	/**
	 * Blocks the calling thread until the future result is available or the specified duration is exceeded.
	 *
	 * @param Duration The maximum time span to wait for the future result.
	 * @return true if the result is available, false otherwise.
	 * @see Wait, WaitUntil
	 */
	bool WaitFor(const FTimespan& Duration) const
	{
		return State.IsValid() ? State->WaitFor(Duration) : false;
	}

	/**
	 * Blocks the calling thread until the future result is available or the specified time is hit.
	 *
	 * @param Time The time until to wait for the future result (in UTC).
	 * @return true if the result is available, false otherwise.
	 * @see Wait, WaitUntil
	 */
	bool WaitUntil(const FDateTime& Time) const
	{
		return WaitFor(Time - FDateTime::UtcNow());
	}

protected:

	/** Default constructor. */
	TFutureBase() = default;

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InState The shared state to initialize with.
	 */
	TFutureBase(TSharedPtr<StateType>&& InState)
		: State(MoveTemp(InState))
	{
	}
	TFutureBase(const TSharedPtr<StateType>& InState)
		: State(InState)
	{
	}

	/**
	 * Gets the shared state object.
	 *
	 * @return The shared state object.
	 */
	const TSharedPtr<StateType>& GetState() const
	{
		// if you hit this assertion then your future has an invalid state.
		// this happens if you have an uninitialized future or if you moved
		// it to another instance.
		check(State.IsValid());

		return State;
	}

	/**
	 * Set a completion callback that will be called once the future completes
	 *	or immediately if already completed
	 *
	 * @param Continuation a continuation taking an argument of type TFuture<ResultType>
	 * @return nothing at the moment but could return another future to allow future chaining
	 */
	template<typename Func>
	auto Then(Func Continuation);

	/**
	 * Convenience wrapper for Then that
	 *	set a completion callback that will be called once the future completes
	 *	or immediately if already completed
	 * @param Continuation a continuation taking an argument of type ResultType
	 * @return nothing at the moment but could return another future to allow future chaining
	 */
	template<typename Func>
	auto Next(Func Continuation);

	/**
	 * Reset the future.
	 *	Resetting a future removes any continuation from its shared state and invalidates it.
	 *	Useful for discarding yet to be completed future cleanly.
	 */
	void Reset()
	{
		if (IsValid())
		{
			this->State->SetContinuation(nullptr);
			this->State.Reset();
		}
	}

private:

	/** Holds the future's state. */
	TSharedPtr<StateType> State;
};


/**
 * Template for unshared futures.
 */
template<typename ResultType>
class TFuture final
	: public TFutureBase<ResultType>
{
	template <typename>
	friend class TPromise;

	template <typename>
	friend class TFutureBase;

	using BaseType = TFutureBase<ResultType>;

	using MutableResultType = typename BaseType::MutableResultType;
	using ConstResultType   = typename BaseType::ConstResultType;
	using RvalueResultType  = typename BaseType::RvalueResultType;

public:

	/** Default constructor. */
	TFuture() = default;

	// Movable-only
	TFuture(TFuture&&) = default;
	TFuture(const TFuture&) = delete;
	TFuture& operator=(TFuture&&) = default;
	TFuture& operator=(const TFuture&) = delete;
	~TFuture() = default;

	/**
	 * Gets the future's result.
	 *
	 * @return The result as a non-const reference, or the same reference if the future holds a reference, or void if the future holds a void.
	 * @note Not equivalent to std::future::get(). The future remains valid.
	 */
	MutableResultType GetMutable()
		requires (!std::is_object_v<ResultType>)
	{
		return (MutableResultType)this->Get();
	}
	MutableResultType GetMutable() UE_LIFETIMEBOUND
		requires (std::is_object_v<ResultType>)
	{
		return (MutableResultType)this->Get();
	}

	/**
	 * Consumes the future's result and invalidates the future.
	 *
	 * @return The result.
	 * @note Equivalent to std::future::get(). Invalidates the future.
	 */
	ResultType Consume()
	{
		TFuture Local(MoveTemp(*this));
		return (RvalueResultType)Local.Get();
	}

	/**
	 * Moves this future's state into a shared future.
	 *
	 * @return The shared future object.
	 */
	TSharedFuture<ResultType> Share()
	{
		return TSharedFuture<ResultType>(MoveTemp(*this));
	}

	/**
	 * Expose Then functionality
	 * @see TFutureBase 
	 */
	using BaseType::Then;

	/**
	 * Expose Next functionality
	 * @see TFutureBase
	 */
	using BaseType::Next;

	/**
	 * Expose Reset functionality
	 * @see TFutureBase
	 */
	using BaseType::Reset;

private:
	// Forward constructors
	using BaseType::TFutureBase;
};


/* TSharedFuture
*****************************************************************************/

/**
 * Template for shared futures.
 */
template<typename ResultType>
class TSharedFuture final
	: public TFutureBase<ResultType>
{
	template <typename>
	friend class TFuture;

	using BaseType = TFutureBase<ResultType>;

	using MutableResultType = typename BaseType::MutableResultType;
	using ConstResultType   = typename BaseType::ConstResultType;
	using RvalueResultType  = typename BaseType::RvalueResultType;

public:

	/** Default constructor. */
	TSharedFuture() = default;

	/**
	 * Creates and initializes a new instances from a future object.
	 *
	 * @param Future The future object to initialize from.
	 */
	TSharedFuture(TFuture<ResultType>&& Future)
		: BaseType(MoveTemp(Future))
	{ }

	/**
	 * Gets the future's result.
	 *
	 * @return The result as a const reference, or the same reference if the future holds a reference, or void if the future holds a void.
	 * @note Not equivalent to std::future::get(). The future remains valid.
	 */
	ConstResultType Get() const
	{
		// This forwarding function is necessary to 'cancel' the UE_LIFETIMEBOUND of the base class, as
		// other shared futures can keep the object alive.
		return BaseType::Get();
	}

private:
	// Forward constructors
	using BaseType::TFutureBase;
};


/* TPromise
*****************************************************************************/

/**
 * Template for promises.
 */
template<typename ResultType>
class TPromise final
{
	using StateType = TFutureState<ResultType>;

	// This is necessary because we can't form references to void for the parameter of Set().
	using SetType = std::conditional_t<std::is_void_v<ResultType>, int, ResultType>;

public:

	/** Default constructor (creates a new shared state). */
	TPromise()
		: State(MakeShared<TFutureState<ResultType>>())
	{
	}

	/**
	 * Create and initialize a new instance with a callback.
	 *
	 * @param CompletionCallback A function that is called when the future state is completed.
	 */
	TPromise(TUniqueFunction<void()>&& CompletionCallback)
		: State(MakeShared<TFutureState<ResultType>>(MoveTemp(CompletionCallback)))
	{
	}

	// Movable-only
	TPromise(TPromise&& Other) = default;
	TPromise(const TPromise& Other) = delete;
	TPromise& operator=(TPromise&& Other) = default;
	TPromise& operator=(const TPromise& Other) = delete;

	/** Destructor. */
	~TPromise()
	{
		if (State.IsValid())
		{
			// if you hit this assertion then your promise never had its result
			// value set. broken promises are considered programming errors.
			check(State->IsComplete());
		}
	}

	/**
	 * Gets a TFuture object associated with the shared state of this promise.
	 *
	 * @return The TFuture object.
	 */
	TFuture<ResultType> GetFuture()
	{
		check(!bFutureRetrieved);
		bFutureRetrieved = true;

		return TFuture<ResultType>(this->GetState());
	}

	/**
	 * Sets the promised result.
	 *
	 * The result must be set only once. An assertion will
	 * be triggered if this method is called a second time.
	 *
	 * @param Result The result value to set.
	 */
	inline void SetValue(const SetType& Result)
		requires(!std::is_void_v<ResultType>)
	{
		EmplaceValue(Result);
	}
	inline void SetValue(SetType&& Result)
		requires(!std::is_void_v<ResultType> && !std::is_lvalue_reference_v<ResultType>)
	{
		EmplaceValue(MoveTemp(Result));
	}
	inline void SetValue()
		requires(std::is_void_v<ResultType>)
	{
		EmplaceValue();
	}

	/**
	 * Sets the promised result.
	 *
	 * The result must be set only once. An assertion will
	 * be triggered if this method is called a second time.
	 *
	 * @param Args The arguments to forward to the constructor of the result.
	 */
	template <typename... ArgTypes>
	void EmplaceValue(ArgTypes&&... Args)
	{
		this->GetState()->EmplaceResult(Forward<ArgTypes>(Args)...);
	}

protected:
	/**
	 * Gets the shared state object.
	 *
	 * @return The shared state object.
	 */
	const TSharedPtr<StateType>& GetState()
	{
		// if you hit this assertion then your promise has an invalid state.
		// this happens if you move the promise to another instance.
		check(State.IsValid());

		return State;
	}

private:
	/** Holds the shared state object. */
	TSharedPtr<StateType> State;

	/** Whether a future has already been retrieved from this promise. */
	bool bFutureRetrieved = false;
};

/* TFuture::Then
*****************************************************************************/

// Then implementation
template<typename ResultType>
template<typename Func>
auto TFutureBase<ResultType>::Then(Func Continuation) //-> TFuture<decltype(Continuation(MoveTemp(TFuture<ResultType>())))>
{
	check(IsValid());
	using ReturnValue = decltype(Continuation(MoveTemp(TFuture<ResultType>())));

	TPromise<ReturnValue> Promise;
	TFuture<ReturnValue> FutureResult = Promise.GetFuture();
	TUniqueFunction<void()> Callback = [PromiseCapture = MoveTemp(Promise), ContinuationCapture = MoveTemp(Continuation), StateCapture = this->State]() mutable
	{
		if constexpr (std::is_void_v<ReturnValue>)
		{
			ContinuationCapture(TFuture<ResultType>(MoveTemp(StateCapture)));
			PromiseCapture.SetValue();
		}
		else
		{
			PromiseCapture.SetValue(ContinuationCapture(TFuture<ResultType>(MoveTemp(StateCapture))));
		}
	};

	// This invalidate this future.
	TSharedPtr<StateType> MovedState = MoveTemp(this->State);
	MovedState->SetContinuation(MoveTemp(Callback));
	return FutureResult;
}

// Next implementation
template<typename ResultType>
template<typename Func>
auto TFutureBase<ResultType>::Next(Func Continuation) //-> TFuture<decltype(Continuation(Consume()))>
{
	return this->Then([Continuation = MoveTemp(Continuation)](TFuture<ResultType> Self) mutable
	{
		if constexpr (std::is_void_v<ResultType>)
		{
			Self.Consume();
			return Continuation();
		}
		else
		{
			return Continuation(Self.Consume());
		}
	});
}

/** Helper to create and immediately fulfill a promise */
template<typename ResultType, typename... ArgTypes>
TPromise<ResultType> MakeFulfilledPromise(ArgTypes&&... Args)
{
	TPromise<ResultType> Promise;
	Promise.EmplaceValue(Forward<ArgTypes>(Args)...);
	return Promise;
}

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Templates/Requires.h"
#endif

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Templates/Models.h"
#endif
