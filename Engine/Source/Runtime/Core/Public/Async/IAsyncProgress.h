// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Optional.h"
#include "Internationalization/Text.h"

/**
 * Interface for checking the progress of asynchronous tasks.
 */
class IAsyncProgress
{
public:

	/**
	 * Gets the task's completion percentage, if available (0.0 to 1.0, or unset).
	 *
	 * Tasks should make a best effort to calculate a completion percentage. If the
	 * calculation is expensive, the value should be cached and only updated in regular
	 * intervals or whenever is appropriate. If a completion percentage cannot be
	 * computed at all, the optional return value should be unset.
	 *
	 * @return Completion percentage value.
	 * @see GetStatusText
	 */
	virtual TOptional<float> GetCompletion() = 0;

	/**
	 * Gets a human readable text for the task's current status.
	 *
	 * @return Status text.
	 * @see GetCompletion
	 */
	virtual FText GetStatusText() = 0;

public:

	/** A delegate that is executed if the task's progress changed. */
	virtual FSimpleDelegate& OnProgressChanged() = 0;

public:

	/** Virtual destructor. */
	virtual ~IAsyncProgress() = default;
};
