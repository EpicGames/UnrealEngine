// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Enumerates possible states of slow running tasks.
 */
enum class EAsyncTaskState
{
	/** Task has been canceled. */
	Cancelled,

	/** Task has completed execution. */
	Completed,

	/** Task execution failed. */
	Failed,

	/** Task execution is still pending. */
	Pending,

	/** Task is executing. */
	Running,
};


/**
 * Interface for asynchronous tasks.
 *
 * A asynchronous task is a unit of work that runs in parallel to the caller and may take a
 * considerable amount of time to complete, i.e. several seconds, minutes or even hours.
 * This interface provides mechanisms for tracking and canceling such tasks.
 */
class IAsyncTask
{
public:

	/**
	 * Cancel this task.
	 *
	 * If the task is already running, it should make a best effort to abort execution
	 * as soon as possible. This method should be implemented in such a way that it returns
	 * immediately and does not block the caller.
	 */
	virtual void Cancel() = 0;

	/**
	 * Gets the current state of the task.
	 *
	 * @return Task state.
	 */
	virtual EAsyncTaskState GetTaskState() = 0;

public:

	/** Virtual destructor. */
	virtual ~IAsyncTask() = default;
};
