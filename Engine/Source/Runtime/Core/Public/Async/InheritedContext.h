// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/LowLevelMemTracker.h"
#include "ProfilingDebugging/MetadataTrace.h"
#include "ProfilingDebugging/TagTrace.h"
#include "Misc/AppTime.h"

namespace UE
{
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	/** Structure representing the captured LLM Tags */
	struct FLLMActiveTagsCapture
	{
		/** Fixed array of LLMTagSets captured */
		const UE::LLMPrivate::FTagData* LLMTags[static_cast<uint32>(ELLMTagSet::Max)];

		void CaptureActiveTagData()
		{
			for (int32 TagSetIndex = 0; TagSetIndex < static_cast<int32>(ELLMTagSet::Max); ++TagSetIndex)
			{
				LLMTags[TagSetIndex] = FLowLevelMemTracker::IsEnabled() ?
					FLowLevelMemTracker::Get().GetActiveTagData(ELLMTracker::Default, static_cast<ELLMTagSet>(TagSetIndex))
					: nullptr;
			}
		}
	};

	/** Structure holding the captured LLM scopes */
	struct FLLMActiveTagsScope
	{
		/** Fixed size array which holds a copy of the LLM scopes for later recall */
		/** Order doesn't matter so we can use a FixedAllocator and not require a default constructor */
		TArray<FLLMScope, TFixedAllocator<static_cast<uint32>(ELLMTagSet::Max)>> LLMScopes;

		explicit FLLMActiveTagsScope(const FLLMActiveTagsCapture& InActiveTagsCapture)
		{
			CaptureLLMScopes(InActiveTagsCapture);
		}

		void CaptureLLMScopes(const FLLMActiveTagsCapture& InActiveTagsCapture)
		{
			for (int32 TagSetIndex = 0; TagSetIndex < static_cast<int32>(ELLMTagSet::Max); ++TagSetIndex)
			{
				LLMScopes.Emplace(InActiveTagsCapture.LLMTags[TagSetIndex], false /* bIsStatTag */, static_cast<ELLMTagSet>(TagSetIndex), ELLMTracker::Default);
			}
		}
	};
#endif

	/** Opaque per-task storage for generic extension data (defined in .cpp). */
	struct FInheritedContextExtensionData;

	/**
	 * Lightweight, copyable handle for a generic inherited context extension.
	 * Wraps the extension descriptor (callbacks, sizes, UserData) in a TSharedPtr-based pimpl,
	 * so copying the handle bumps the ref count and the descriptor stays alive automatically.
	 */
	struct FInheritedContextExtension
	{
		using FCaptureFunc  = void (*)(void* DataDst, void* UserData);
		using FApplyFunc    = void (*)(const void* DataSrc, void* SaveDst, void* UserData);
		using FRestoreFunc  = void (*)(const void* SaveSrc, void* UserData);
		using FDestroyFunc  = void (*)(const void* DataSrc, void* UserData);

		struct FInterface
		{
			FCaptureFunc  Capture;    // Read current thread state into DataDst
			FApplyFunc    Apply;      // Write DataSrc to thread state, save previous to SaveDst
			FRestoreFunc  Restore;    // Put back previous state from SaveSrc
			FDestroyFunc  Destroy;    // Destruct data in buffer (nullable for trivial types)
			uint32        DataSize;
			uint32        DataAlign;
			void*         UserData;   // Typically pointer to the variable being propagated
		};

		TSharedPtr<FInterface> Impl;

		bool IsValid() const { return Impl.IsValid(); }

		template<typename T>
		T& GetState() const { return *static_cast<T*>(Impl->UserData); }
	};

	/**
	 * Creates an FInheritedContextExtension for a variable accessed through a callable.
	 * The callable is invoked on the current thread each time, so this works correctly with thread_local variables.
	 * Captures by copy-constructing, applies by assignment. Equivalent to TGuardValue propagation.
	 * The callable must return an lvalue reference and must be convertible to a function pointer (i.e. non-capturing lambda or free function).
	 *
	 * Usage:
	 *   static FInheritedContextExtension GMyFlagExt = MakeInheritedContextExtension([]() -> bool& { return GMyFlag; });
	 */
	template<typename Func>
	FInheritedContextExtension MakeInheritedContextExtension(Func GetVar)
	{
		using ReturnType = decltype(GetVar());
		static_assert(std::is_lvalue_reference_v<ReturnType>, "Accessor must return an lvalue reference (e.g. []() -> bool& { return GMyFlag; })");
		using T = std::remove_reference_t<ReturnType>;
		using FuncPtr = T& (*)();

		static_assert(sizeof(FuncPtr) <= sizeof(void*), "Function pointer must fit in void*");

		FuncPtr FnPtr = GetVar;

		FInheritedContextExtension Ext;
		Ext.Impl = MakeShared<FInheritedContextExtension::FInterface>();
		Ext.Impl->Capture = [](void* DataDst, void* UserData)
		{
			auto Fn = reinterpret_cast<T& (*)()>(UserData);
			new (DataDst) T(Fn());
		};
		Ext.Impl->Apply = [](const void* DataSrc, void* SaveDst, void* UserData)
		{
			auto Fn = reinterpret_cast<T& (*)()>(UserData);
			new (SaveDst) T(Fn());
			Fn() = *static_cast<const T*>(DataSrc);
		};
		Ext.Impl->Restore = [](const void* SaveSrc, void* UserData)
		{
			auto Fn = reinterpret_cast<T& (*)()>(UserData);
			Fn() = *static_cast<const T*>(SaveSrc);
		};
		Ext.Impl->Destroy = [](const void* DataSrc, void* /*UserData*/)
		{
			static_cast<const T*>(DataSrc)->~T();
		};
		Ext.Impl->DataSize  = sizeof(T);
		Ext.Impl->DataAlign = alignof(T);
		Ext.Impl->UserData  = reinterpret_cast<void*>(FnPtr);
		return Ext;
	}

	/**
	 * Creates an FInheritedContextExtension for a getter/setter API.
	 * Use when the variable is not directly accessible (e.g. accessed through IsSavingPackage()/SetIsSavingPackage()).
	 * Getter and Setter are compile-time function pointers baked into the generated callbacks.
	 *
	 * Usage:
	 *   static FInheritedContextExtension GExt = MakeInheritedContextExtension<&UE::IsSavingPackage, &UE::SetIsSavingPackage>();
	 */
	template<auto Getter, auto Setter>
	FInheritedContextExtension MakeInheritedContextExtension()
	{
		using T = std::decay_t<decltype(Getter())>;

		FInheritedContextExtension Ext;
		Ext.Impl = MakeShared<FInheritedContextExtension::FInterface>();
		Ext.Impl->Capture = [](void* DataDst, void* /*UserData*/)
		{
			new (DataDst) T(Getter());
		};
		Ext.Impl->Apply = [](const void* DataSrc, void* SaveDst, void* /*UserData*/)
		{
			new (SaveDst) T(Getter());
			Setter(*static_cast<const T*>(DataSrc));
		};
		Ext.Impl->Restore = [](const void* SaveSrc, void* /*UserData*/)
		{
			Setter(*static_cast<const T*>(SaveSrc));
		};
		Ext.Impl->Destroy = [](const void* DataSrc, void* /*UserData*/)
		{
			static_cast<const T*>(DataSrc)->~T();
		};
		Ext.Impl->DataSize  = sizeof(T);
		Ext.Impl->DataAlign = alignof(T);
		Ext.Impl->UserData  = nullptr; // Getter/Setter are baked into template callbacks; UserData is not needed
		return Ext;
	}

	/**
	 * Derived FInterface that co-allocates state of type T with the descriptor.
	 * Defined at namespace scope because local classes cannot have template members.
	 * When the TSharedPtr<FInterface> ref count hits zero, the control block destructs
	 * TStatefulContextExtensionImpl<T> which properly destroys State via the type-erased
	 * deleter stored by TSharedPtr. No virtual destructor needed.
	 */
	template<typename T>
	struct TStatefulContextExtensionImpl : FInheritedContextExtension::FInterface
	{
		T State;

		template<typename... CtorArgs>
		explicit TStatefulContextExtensionImpl(CtorArgs&&... InArgs) : State(Forward<CtorArgs>(InArgs)...) {}
	};

	/**
	 * Creates an FInheritedContextExtension with custom scope callbacks and co-allocated mutable state.
	 * The state of type T is constructed in-place inside a derived FInterface and owned by the handle's
	 * TSharedPtr, so it lives exactly as long as the last copy of the extension handle.
	 * ApplyFn() is called when a task starts executing and returns saved state (preserved per-scope).
	 * RestoreFn(SavedState, SharedState) is called when the task scope ends.
	 * Access the state after construction via Ext.GetState<T>().
	 *
	 * Usage (timing accumulation):
	 *   double OnStartTiming() { return FPlatformTime::Seconds(); }
	 *   void OnStopTiming(const double& Start, std::atomic<double>& Acc) {
	 *       Acc.fetch_add(FPlatformTime::Seconds() - Start, std::memory_order_relaxed);
	 *   }
	 *   auto Ext = MakeStatefulInheritedContextExtension<&OnStartTiming, &OnStopTiming, std::atomic<double>>(0.0);
	 *   FInheritedContextExtensionScope Scope(Ext);
	 *   // ... launch tasks ...
	 *   double Result = Ext.GetState<std::atomic<double>>().load();
	 */
	template<auto ApplyFn, auto RestoreFn, typename T, typename... ArgsType>
	FInheritedContextExtension MakeStatefulInheritedContextExtension(ArgsType&&... Args)
	{
		using SavedState = std::decay_t<decltype(ApplyFn())>;

		// The capture block stores a T* and the saved state block stores SavedState.
		// They are separate allocations but share the same DataSize/DataAlign layout,
		// so size must accommodate the larger of the two.
		constexpr uint32 Size  = static_cast<uint32>(sizeof(T*) > sizeof(SavedState) ? sizeof(T*) : sizeof(SavedState));
		constexpr uint32 Align = static_cast<uint32>(alignof(T*) > alignof(SavedState) ? alignof(T*) : alignof(SavedState));

		using FStatefulImpl = TStatefulContextExtensionImpl<T>;
		auto StatefulRef = MakeShared<FStatefulImpl>(Forward<ArgsType>(Args)...);

		StatefulRef->Capture = [](void* DataDst, void* UserData)
		{
			// Store the shared state pointer so it propagates through the chain
			*static_cast<T**>(DataDst) = static_cast<T*>(UserData);
		};
		StatefulRef->Apply = [](const void* /*DataSrc*/, void* SaveDst, void* /*UserData*/)
		{
			// Call ApplyFn to begin scope (e.g. record start time)
			new (SaveDst) SavedState(ApplyFn());
		};
		StatefulRef->Restore = [](const void* SaveSrc, void* UserData)
		{
			// Call RestoreFn with saved state and shared state
			RestoreFn(*static_cast<const SavedState*>(SaveSrc), *static_cast<T*>(UserData));
			static_cast<const SavedState*>(SaveSrc)->~SavedState();
		};
		StatefulRef->Destroy  = nullptr; // Captured data is T* (trivially destructible)
		StatefulRef->DataSize  = Size;
		StatefulRef->DataAlign = Align;
		StatefulRef->UserData  = &StatefulRef->State;

		FInheritedContextExtension Ext;
		Ext.Impl = TSharedPtr<FInheritedContextExtension::FInterface>(StatefulRef);
		return Ext;
	}

	/**
	 * RAII scope that activates an extension for capture.
	 * While alive, any CaptureInheritedContext() on the same thread includes this extension.
	 */
	class FInheritedContextExtensionScope
	{
	public:
		UE_NONCOPYABLE(FInheritedContextExtensionScope);

		CORE_API explicit FInheritedContextExtensionScope(const FInheritedContextExtension& InExtension);
		CORE_API ~FInheritedContextExtensionScope();

	private:
		friend class FInheritedContextBase;

		FInheritedContextExtension Extension;
		FInheritedContextExtensionScope* Next;
	};

	// Restores an inherited context for the current scope.
	// An instance must be obtained by calling `FInheritedContextBase::RestoreInheritedContext()`
	class FInheritedContextScope
	{
	public:
		CORE_API ~FInheritedContextScope();

	private:
		UE_NONCOPYABLE(FInheritedContextScope);

		friend class FInheritedContextBase; // allow construction only by `FInheritedContextBase`

		CORE_API FInheritedContextScope(TSharedPtr<const FAppTime> AppTime
		#if ENABLE_LOW_LEVEL_MEM_TRACKER
			, const FLLMActiveTagsCapture& InInheritedLLMTag
		#endif
		#if (UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED)
			, int32 InInheritedMemTag
		#endif
		#if (UE_TRACE_METADATA_ENABLED && UE_TRACE_ENABLED)
			, uint32 InInheritedMetadataId
		#endif
			, FInheritedContextExtensionData* InExtensionData
		);

		TSharedPtr<const FAppTime> PrevAppTime;

	#if ENABLE_LOW_LEVEL_MEM_TRACKER
		FLLMActiveTagsScope LLMScopes;
	#endif

	#if (UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED)
		FMemScope MemScope;
	#endif

	#if (UE_TRACE_METADATA_ENABLED && UE_TRACE_ENABLED)
		FMetadataRestoreScope MetaScope;
	#endif

		FInheritedContextExtensionData* ExtensionSavedState = nullptr;  // Previous thread state saved during Apply, consumed by Restore
		FInheritedContextExtensionData* PrevPropagatedData = nullptr;   // Saved GPropagatedExtensionData, restored when scope ends
	};

	// this class extends the inherited context (see private members for what the inherited context is) to cover async execution.
	// Is intended to be used as a base class, if the inherited context is compiled out it takes 0 space
	class FInheritedContextBase
	{
	public:
		FInheritedContextBase() = default;
		CORE_API ~FInheritedContextBase();

		FInheritedContextBase(const FInheritedContextBase&) = delete;
		FInheritedContextBase& operator=(const FInheritedContextBase&) = delete;

		FInheritedContextBase(FInheritedContextBase&& Other)
			: AppTime(MoveTemp(Other.AppTime))
#if ENABLE_LOW_LEVEL_MEM_TRACKER
			, InheritedLLMTags(Other.InheritedLLMTags)
#endif
#if UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED
			, InheritedMemTag(Other.InheritedMemTag)
#endif
#if UE_TRACE_METADATA_ENABLED && UE_TRACE_ENABLED
			, InheritedMetadataId(Other.InheritedMetadataId)
#endif
			, ExtensionData(Other.ExtensionData)
		{
			Other.ExtensionData = nullptr;
		}

		CORE_API FInheritedContextBase& operator=(FInheritedContextBase&& Other);

		// must be called in the inherited context, e.g. on launching an async task
		CORE_API void CaptureInheritedContext();

		// must be called where the inherited context should be restored, e.g. at the start of an async task execution
		[[nodiscard]] CORE_API FInheritedContextScope RestoreInheritedContext();

	private:
		TSharedPtr<const FAppTime> AppTime;

#if ENABLE_LOW_LEVEL_MEM_TRACKER
		FLLMActiveTagsCapture InheritedLLMTags;
#endif

#if UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED
		int32 InheritedMemTag;
#endif

#if UE_TRACE_METADATA_ENABLED && UE_TRACE_ENABLED
		uint32 InheritedMetadataId;
#endif

		FInheritedContextExtensionData* ExtensionData = nullptr;
	};
}
