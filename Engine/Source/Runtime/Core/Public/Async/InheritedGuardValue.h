// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/InheritedContext.h"

/**
 * Combines TGuardValue with FInheritedContextExtension + FInheritedContextExtensionScope.
 * Sets a value for the current scope (like TGuardValue) AND propagates it to async tasks
 * launched within the scope (via the inherited context extension mechanism).
 *
 * Usage with callable accessor (creates extension internally):
 *   static thread_local bool GMyFlag = false;
 *   TInheritedGuardValue<bool> Guard([]() -> bool& { return GMyFlag; }, true);
 *
 * Usage with pre-built extension (avoids allocation per guard):
 *   static auto GExt = UE::MakeInheritedContextExtension([]() -> bool& { return GMyFlag; });
 *   TInheritedGuardValue<bool> Guard(GExt, GMyFlag, true);
 */
template<typename RefType, typename AssignedType = RefType>
struct TInheritedGuardValue : private FNoncopyable
{
	/**
	 * Callable constructor — creates the extension internally.
	 * GetVar must return RefType& (e.g. []() -> bool& { return GMyFlag; }).
	 * Allocates a TSharedPtr<FInterface> internally, so prefer the pre-built variant for hot paths.
	 */
	template<typename Func>
	[[nodiscard]] TInheritedGuardValue(Func GetVar, const AssignedType& NewValue)
		: Extension(UE::MakeInheritedContextExtension(GetVar))
		, Scope(Extension)
		, RefValue(GetVar())
		, OriginalValue(RefValue)
	{
		RefValue = NewValue;
	}

	/**
	 * Pre-built extension constructor — reuses an existing FInheritedContextExtension.
	 * No allocation; just copies the handle (bumps ref count).
	 * The extension must have been created with MakeInheritedContextExtension(Func).
	 */
	[[nodiscard]] TInheritedGuardValue(const UE::FInheritedContextExtension& InExtension, RefType& ReferenceValue, const AssignedType& NewValue)
		: Extension(InExtension)
		, Scope(Extension)
		, RefValue(ReferenceValue)
		, OriginalValue(ReferenceValue)
	{
		RefValue = NewValue;
	}

	~TInheritedGuardValue()
	{
		RefValue = OriginalValue;
	}

	/**
	 * Provides read-only access to the original value of the data being tracked by this struct
	 *
	 * @return	a const reference to the original data value
	 */
	UE_FORCEINLINE_HINT const AssignedType& GetOriginalValue() const
	{
		return OriginalValue;
	}

private:
	UE::FInheritedContextExtension Extension;
	UE::FInheritedContextExtensionScope Scope;
	RefType& RefValue;
	AssignedType OriginalValue;
};

/**
 * Combines TGuardValueAccessors with FInheritedContextExtension + FInheritedContextExtensionScope.
 * Like TInheritedGuardValue but for variables accessed through getter/setter functions rather than
 * direct references. Getter and Setter are compile-time function pointers baked into the extension
 * callbacks — no TFunction overhead.
 *
 * Usage (creates extension internally):
 *   TInheritedGuardValueAccessors<&UE::IsSavingPackage, &UE::SetIsSavingPackage> Guard(true);
 *
 * Usage with pre-built extension (avoids allocation per guard):
 *   static auto GExt = UE::MakeInheritedContextExtension<&UE::IsSavingPackage, &UE::SetIsSavingPackage>();
 *   TInheritedGuardValueAccessors<&UE::IsSavingPackage, &UE::SetIsSavingPackage> Guard(GExt, true);
 */
template<auto Getter, auto Setter>
struct TInheritedGuardValueAccessors : private FNoncopyable
{
	using ValueType = std::decay_t<decltype(Getter())>;

	/**
	 * Creates the extension internally.
	 * Allocates a TSharedPtr<FInterface> internally, so prefer the pre-built variant for hot paths.
	 */
	[[nodiscard]] explicit TInheritedGuardValueAccessors(const ValueType& NewValue)
		: Extension(UE::MakeInheritedContextExtension<Getter, Setter>())
		, Scope(Extension)
		, OriginalValue(Getter())
	{
		Setter(NewValue);
	}

	/**
	 * Pre-built extension constructor — reuses an existing FInheritedContextExtension.
	 * No allocation; just copies the handle (bumps ref count).
	 * The extension must have been created with MakeInheritedContextExtension<Getter, Setter>().
	 */
	[[nodiscard]] TInheritedGuardValueAccessors(const UE::FInheritedContextExtension& InExtension, const ValueType& NewValue)
		: Extension(InExtension)
		, Scope(Extension)
		, OriginalValue(Getter())
	{
		Setter(NewValue);
	}

	~TInheritedGuardValueAccessors()
	{
		Setter(OriginalValue);
	}

	/**
	 * Provides read-only access to the original value of the data being tracked by this struct
	 *
	 * @return	a const reference to the original data value
	 */
	UE_FORCEINLINE_HINT const ValueType& GetOriginalValue() const
	{
		return OriginalValue;
	}

private:
	UE::FInheritedContextExtension Extension;
	UE::FInheritedContextExtensionScope Scope;
	ValueType OriginalValue;
};
