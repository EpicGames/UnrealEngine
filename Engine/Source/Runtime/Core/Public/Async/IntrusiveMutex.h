// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Fundamental/Oversubscription.h"
#include "Async/ParkingLot.h"
#include "Concepts/DecaysTo.h"
#include "Concepts/Integral.h"
#include "CoreTypes.h"
#include "HAL/PlatformProcess.h"

#include <atomic>
#include <type_traits>

namespace UE
{

template <typename ParamsType>
using TIntrusiveMutexStateType_T = std::decay_t<decltype(ParamsType::IsLockedFlag)>;

template <typename ParamsType, typename StateType = TIntrusiveMutexStateType_T<ParamsType>>
concept CIntrusiveMutexParams = requires
{
	requires CIntegral<StateType>;

	// Required: constexpr static StateType IsLockedFlag = ...;
	// Flag that is set in the state when the mutex is locked.
	{ ParamsType::IsLockedFlag } -> CDecaysTo<StateType>;

	// Required: constexpr static StateType MayHaveWaitingLockFlag = ...;
	// Flag that is set in the state when a thread may be waiting to lock the mutex.
	{ ParamsType::MayHaveWaitingLockFlag } -> CDecaysTo<StateType>;

	// Optional: constexpr static StateType IsLockedMask = ...;
	// Mask tested against State to determine if the mutex is locked. Defaults to IsLockedFlag.
	requires
		(!requires { ParamsType::IsLockedMask; }) ||
		requires { { ParamsType::IsLockedMask } -> CDecaysTo<StateType>; };

	// Optional: constexpr static int32 SpinLimit = ...;
	// Maximum number of times to spin before waiting. Defaults to 40.
	requires
		(!requires { ParamsType::SpinLimit; }) ||
		requires { { ParamsType::SpinLimit } -> CDecaysTo<int32>; };

	// Optional: static const void* GetWaitAddress(std::atomic<StateType>& State);
	// Returns the address to pass to ParkingLot wait and wake functions. Defaults to &State.
	requires
		(!requires { ParamsType::GetWaitAddress; }) ||
		requires(std::atomic<StateType>& State) { { ParamsType::GetWaitAddress(State) } -> CDecaysTo<const void*>; };
};

/**
 * A 2-bit intrusive mutex that is not fair and does not support recursive locking.
 *
 * All bits of the state referenced by IsLockedFlag, IsLockedMask, and MayHaveWaitingLockFlag
 * must be initialized to 0 or to values that are consistent with the functions being called.
 */
template <CIntrusiveMutexParams ParamsType>
class TIntrusiveMutex
{
	TIntrusiveMutex() = delete;

	using StateType = TIntrusiveMutexStateType_T<ParamsType>;

	constexpr static StateType IsLockedFlag = ParamsType::IsLockedFlag;
	constexpr static StateType MayHaveWaitingLockFlag = ParamsType::MayHaveWaitingLockFlag;

	constexpr static StateType IsLockedMask = []
	{
		if constexpr (requires { ParamsType::IsLockedMask; })
		{
			return ParamsType::IsLockedMask;
		}
		else
		{
			return ParamsType::IsLockedFlag;
		}
	}();

	constexpr static int32 SpinLimit = []
	{
		if constexpr (requires { ParamsType::SpinLimit; })
		{
			return ParamsType::SpinLimit;
		}
		else
		{
			return 40;
		}
	}();

	static_assert(IsLockedFlag && (IsLockedFlag & (IsLockedFlag - 1)) == 0, "IsLockedFlag must be one bit.");
	static_assert(MayHaveWaitingLockFlag && (MayHaveWaitingLockFlag & (MayHaveWaitingLockFlag - 1)) == 0, "MayHaveWaitingLockFlag must be one bit.");
	static_assert(IsLockedFlag != MayHaveWaitingLockFlag, "IsLockedFlag and MayHaveWaitingLockFlag must be different bits.");
	static_assert((IsLockedMask & IsLockedFlag) == IsLockedFlag, "IsLockedMask must contain IsLockedFlag.");
	static_assert((IsLockedMask & MayHaveWaitingLockFlag) == 0, "IsLockedMask must not contain MayHaveWaitingLockFlag.");
	static_assert(SpinLimit >= 0, "SpinLimit must be non-negative.");

	FORCEINLINE static const void* GetWaitAddress(const std::atomic<StateType>& State)
	{
		if constexpr (requires { ParamsType::GetWaitAddress; })
		{
			return ParamsType::GetWaitAddress(State);
		}
		else
		{
			return &State;
		}
	}

public:
	[[nodiscard]] FORCEINLINE static bool IsLocked(const std::atomic<StateType>& State)
	{
		return !!(State.load(std::memory_order_relaxed) & IsLockedFlag);
	}

	[[nodiscard]] FORCEINLINE static bool TryLock(std::atomic<StateType>& State)
	{
		StateType Expected = State.load(std::memory_order_relaxed);
		return !(Expected & IsLockedMask) &&
			State.compare_exchange_strong(Expected, Expected | IsLockedFlag, std::memory_order_acquire, std::memory_order_relaxed);
	}

	FORCEINLINE static void Lock(std::atomic<StateType>& State)
	{
		StateType Expected = State.load(std::memory_order_relaxed) & ~IsLockedMask & ~MayHaveWaitingLockFlag;
		if (LIKELY(State.compare_exchange_weak(Expected, Expected | IsLockedFlag, std::memory_order_acquire, std::memory_order_relaxed)))
		{
			return;
		}
		LockSlow(State);
	}

	FORCEINLINE static void LockLoop(std::atomic<StateType>& State)
	{
		int32 SpinCount = 0;
		for (StateType CurrentState = State.load(std::memory_order_relaxed);;)
		{
			// Try to acquire the lock if it was unlocked, even if there are waiting threads.
			// Acquiring the lock despite the waiting threads means that this lock is not FIFO and thus not fair.
			if (LIKELY(!(CurrentState & IsLockedMask)))
			{
				if (LIKELY(State.compare_exchange_weak(CurrentState, CurrentState | IsLockedFlag, std::memory_order_acquire, std::memory_order_relaxed)))
				{
					return;
				}
				continue;
			}

			// Spin up to the spin limit while there are no waiting threads.
			if (LIKELY(!(CurrentState & MayHaveWaitingLockFlag) && SpinCount < SpinLimit))
			{
				FPlatformProcess::Yield();
				++SpinCount;
				CurrentState = State.load(std::memory_order_relaxed);
				continue;
			}

			// Store that there are waiting threads. Restart if the state has changed since it was loaded.
			if (LIKELY(!(CurrentState & MayHaveWaitingLockFlag)))
			{
				if (UNLIKELY(!State.compare_exchange_weak(CurrentState, CurrentState | MayHaveWaitingLockFlag, std::memory_order_relaxed)))
				{
					continue;
				}
				CurrentState |= MayHaveWaitingLockFlag;
			}

			// Do not enter oversubscription during a wait on a mutex since the wait is generally too short
			// for it to matter and it can worsen performance a lot for heavily contended locks.
			LowLevelTasks::Private::FOversubscriptionAllowedScope _(false);

			// Wait if the state has not changed. Either way, loop back and try to acquire the lock after trying to wait.
			ParkingLot::Wait(GetWaitAddress(State), [&State]
			{
				const StateType NewState = State.load(std::memory_order_relaxed);
				return (NewState & IsLockedMask) && (NewState & MayHaveWaitingLockFlag);
			}, nullptr);
			CurrentState = State.load(std::memory_order_relaxed);
		}
	}

	FORCEINLINE static void Unlock(std::atomic<StateType>& State)
	{
		// Unlock immediately to allow other threads to acquire the lock while this thread looks for a thread to wake.
		const StateType LastState = State.fetch_sub(IsLockedFlag, std::memory_order_release);
		if (LIKELY(!(LastState & MayHaveWaitingLockFlag)))
		{
			return;
		}
		UnlockSlow(State);
	}

	FORCEINLINE static void WakeWaitingThread(std::atomic<StateType>& State)
	{
		ParkingLot::WakeOne(GetWaitAddress(State), [&State](ParkingLot::FWakeState WakeState) -> uint64
		{
			if (!WakeState.bDidWake)
			{
				// Keep the flag until no thread wakes, otherwise shared locks may win before
				// an exclusive lock has a chance.
				State.fetch_and(~MayHaveWaitingLockFlag, std::memory_order_relaxed);
			}
			return 0;
		});
	}

	[[nodiscard]] FORCEINLINE static bool TryWakeWaitingThread(std::atomic<StateType>& State)
	{
		bool bDidWake = false;
		ParkingLot::WakeOne(GetWaitAddress(State), [&State, &bDidWake](ParkingLot::FWakeState WakeState) -> uint64
		{
			if (!WakeState.bDidWake)
			{
				// Keep the flag until no thread wakes, otherwise shared locks may win before
				// an exclusive lock has a chance.
				State.fetch_and(~MayHaveWaitingLockFlag, std::memory_order_relaxed);
			}
			bDidWake = WakeState.bDidWake;
			return 0;
		});
		return bDidWake;
	}

private:
	FORCENOINLINE static void LockSlow(std::atomic<StateType>& State)
	{
		LockLoop(State);
	}

	FORCENOINLINE static void UnlockSlow(std::atomic<StateType>& State)
	{
		WakeWaitingThread(State);
	}
};

} // UE
