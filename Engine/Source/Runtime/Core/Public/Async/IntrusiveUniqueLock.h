// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/IntrusiveMutex.h"

#include <atomic>

namespace UE
{

/**
 * An intrusive mutex wrapper that locks on construction and unlocks on destruction.
 * For details on how to set up an intrusive mutex, see the IntrusiveMutex.h header.
 */
template <UE::CIntrusiveMutexParams ParamsType>
class TIntrusiveUniqueLock
{
	using StateType = UE::TIntrusiveMutexStateType_T<ParamsType>;

public:
	TIntrusiveUniqueLock(TIntrusiveUniqueLock& NoCopyConstruction) = delete;
	TIntrusiveUniqueLock& operator=(TIntrusiveUniqueLock& NoAssignment) = delete;

	[[nodiscard]] explicit TIntrusiveUniqueLock(std::atomic<StateType>& InState)
		: State(InState)
	{
		UE::TIntrusiveMutex<ParamsType>::Lock(State);
	}

	~TIntrusiveUniqueLock()
	{
		UE::TIntrusiveMutex<ParamsType>::Unlock(State);
	}

private:
	std::atomic<StateType>& State;
};

} // namespace UE
