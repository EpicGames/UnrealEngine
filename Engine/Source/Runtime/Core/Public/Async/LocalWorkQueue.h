// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Async/Fundamental/Task.h"
#include "Async/Fundamental/Scheduler.h"
#include "Async/EventCount.h"
#include "Async/TaskGraphInterfaces.h"
#include "Experimental/Containers/FAAArrayQueue.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "Templates/RefCounting.h"
#include <atomic>

template<typename LAMBDA>
class TYCombinator
{
	LAMBDA Lambda;

public:
	constexpr TYCombinator(LAMBDA&& InLambda): Lambda(MoveTemp(InLambda)) 
	{}

	constexpr TYCombinator(const LAMBDA& InLambda): Lambda(InLambda) 
	{}

	template<typename... ARGS>
	constexpr auto operator()(ARGS&&... Args) const -> decltype(Lambda(static_cast<const TYCombinator<LAMBDA>&>(*this), Forward<ARGS>(Args)...))
	{
		return Lambda(static_cast<const TYCombinator<LAMBDA>&>(*this), Forward<ARGS>(Args)...);
	}

	template<typename... ARGS>
	constexpr auto operator()(ARGS&&... Args) -> decltype(Lambda(static_cast<TYCombinator<LAMBDA>&>(*this), Forward<ARGS>(Args)...))
	{
		return Lambda(static_cast<TYCombinator<LAMBDA>&>(*this), Forward<ARGS>(Args)...);
	}
};

template<typename LAMBDA>
constexpr auto MakeYCombinator(LAMBDA&& Lambda)
{
	return TYCombinator<std::decay_t<LAMBDA>>(Forward<LAMBDA>(Lambda));
}

template<typename TaskType>
class TLocalWorkQueue
{
	UE_NONCOPYABLE(TLocalWorkQueue);

	struct FInternalData : public TConcurrentLinearObject<FInternalData, FTaskGraphBlockAllocationTag>, public FRefCountedObject
	{
		FAAArrayQueue<TaskType> TaskQueue;
		std::atomic_int         ActiveWorkers {0};
		std::atomic_bool        CheckDone {false};
		UE::FEventCount         FinishedEvent;

		~FInternalData()
		{
			check(ActiveWorkers == 0);
			check(TaskQueue.dequeue() == nullptr);
		}
	};

	TRefCountPtr<FInternalData> InternalData;
	LowLevelTasks::ETaskPriority Priority;
	TFunctionRef<void(TaskType*)>* DoWork = nullptr;

public:
	inline TLocalWorkQueue(TaskType* InitialWork, LowLevelTasks::ETaskPriority InPriority = LowLevelTasks::ETaskPriority::Count) : Priority(InPriority)
	{
		if (Priority == LowLevelTasks::ETaskPriority::Count)
		{
			const LowLevelTasks::FTask* ActiveTask = LowLevelTasks::FTask::GetActiveTask();
			if (ActiveTask)
			{
				Priority = ActiveTask->GetPriority();
				if (Priority == LowLevelTasks::ETaskPriority::BackgroundLow)
				{
					Priority = LowLevelTasks::ETaskPriority::BackgroundNormal;
				}
				else if (Priority == LowLevelTasks::ETaskPriority::BackgroundNormal)
				{
					Priority = LowLevelTasks::ETaskPriority::BackgroundHigh;
				}
			}
			else
			{
				Priority = LowLevelTasks::ETaskPriority::Default;
			}
		}

		InternalData = new FInternalData();

		AddTask(InitialWork);
	}

public:
	inline void AddTask(TaskType* NewWork)
	{
		check(!InternalData->CheckDone.load(std::memory_order_relaxed));
		InternalData->TaskQueue.enqueue(NewWork);
	}

	inline void AddWorkers(uint16 NumWorkers)
	{
		check(!InternalData->CheckDone.load(std::memory_order_relaxed));
		check(DoWork != nullptr);

		for (uint16 Index = 0; Index < NumWorkers; Index++)
		{
			using FTaskHandle = TSharedPtr<LowLevelTasks::FTask, ESPMode::ThreadSafe>;
			FTaskHandle TaskHandle = MakeShared<LowLevelTasks::FTask, ESPMode::ThreadSafe>();

			TFunctionRef<void(TaskType*)>* LocalDoWork = DoWork;
			TaskHandle->Init(TEXT("TLocalWorkQueue::AddWorkers"), Priority, [LocalDoWork, InternalData = InternalData, TaskHandle]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(TLocalWorkQueue::AddWorkers);
				InternalData->ActiveWorkers.fetch_add(1, std::memory_order_acquire);
				while (TaskType* Work = InternalData->TaskQueue.dequeue())
				{
					check(!InternalData->CheckDone.load(std::memory_order_relaxed));
					(*LocalDoWork)(Work);
				}
				if (InternalData->ActiveWorkers.fetch_sub(1, std::memory_order_release) == 1)
				{
					InternalData->FinishedEvent.Notify();
				}
			});
			verify(TryLaunch(*TaskHandle, LowLevelTasks::EQueuePreference::GlobalQueuePreference));
		}
	}

	inline void Run(TFunctionRef<void(TaskType*)> InDoWork)
	{
		DoWork = &InDoWork;

		TRACE_CPUPROFILER_EVENT_SCOPE(TLocalWorkQueue::Run);

		while (true)
		{
			bool bNoActiveWorkers = InternalData->ActiveWorkers.load(std::memory_order_acquire) == 0;
			if (TaskType* Work = InternalData->TaskQueue.dequeue())
			{
				InDoWork(Work);
			}
			else if (bNoActiveWorkers && InternalData->ActiveWorkers.load(std::memory_order_acquire) == 0)
			{
				break;
			}
			else
			{
				auto Token = InternalData->FinishedEvent.PrepareWait();
				if (InternalData->ActiveWorkers.load(std::memory_order_acquire) == 0)
				{
					continue;
				}

				TRACE_CPUPROFILER_EVENT_SCOPE(TLocalWorkQueue::WaitingForWorkers);
				InternalData->FinishedEvent.Wait(Token);
			}
		}

		InternalData->CheckDone.store(true);
		check(InternalData->TaskQueue.dequeue() == nullptr);
	}
};