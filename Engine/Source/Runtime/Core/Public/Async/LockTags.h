// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE
{

/** Use to acquire a lock on construction. */
inline constexpr struct FAcquireLock final { explicit FAcquireLock() = default; } AcquireLock;

/** Use with dynamic locks to defer locking on construction. */
inline constexpr struct FDeferLock final { explicit FDeferLock() = default; } DeferLock;

} // UE
