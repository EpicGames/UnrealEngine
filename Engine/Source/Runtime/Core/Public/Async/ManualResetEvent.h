// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/ParkingLot.h"
#include <atomic>

namespace UE
{

/**
 * A type of event that remains notified until manually reset.
 */
class FManualResetEvent
{
public:
	constexpr FManualResetEvent() = default;

	FManualResetEvent(const FManualResetEvent&) = delete;
	FManualResetEvent& operator=(const FManualResetEvent&) = delete;

	/**
	 * Returns true if the event is notified, otherwise false.
	 */
	inline bool IsNotified() const
	{
		return !!(State.load(std::memory_order_acquire) & IsNotifiedFlag);
	}

	/**
	 * Wait until the event is notified.
	 */
	inline void Wait()
	{
		WaitUntil(FMonotonicTimePoint::Infinity());
	}

	/**
	 * Wait until the event is notified.
	 *
	 * @param WaitTime   Relative time after which waiting is automatically canceled and the thread wakes.
	 * @return True if the event was notified before the wait time elapsed, otherwise false.
	 */
	inline bool WaitFor(FMonotonicTimeSpan WaitTime)
	{
		return WaitTime.IsZero() ? IsNotified() : WaitUntil(FMonotonicTimePoint::Now() + WaitTime);
	}

	/**
	 * Wait until the event is notified.
	 *
	 * @param WaitTime   Absolute time after which waiting is automatically canceled and the thread wakes.
	 * @return True if the event was notified before the wait time elapsed, otherwise false.
	 */
	inline bool WaitUntil(FMonotonicTimePoint WaitTime)
	{
		for (;;)
		{
			uint8 CurrentState = State.load(std::memory_order_acquire);
			if (CurrentState & IsNotifiedFlag)
			{
				return true;
			}
			if (!WaitTime.IsInfinity() && WaitTime <= FMonotonicTimePoint::Now())
			{
				return false;
			}
			if ((CurrentState & MaybeWaitingFlag) ||
				State.compare_exchange_weak(CurrentState, MaybeWaitingFlag, std::memory_order_acq_rel))
			{
				ParkingLot::WaitUntil(&State, [this]
				{
					uint8 NewState = State.load(std::memory_order_relaxed);
					return !(NewState & IsNotifiedFlag) && (NewState & MaybeWaitingFlag);
				}, nullptr, WaitTime);
			}
		}
	}

	/**
	 * Notifies all waiting threads and leaves the event notified until the next call to Reset().
	 */
	inline void Notify()
	{
		const uint8 CurrentState = State.exchange(IsNotifiedFlag, std::memory_order_release);
		if (!(CurrentState & IsNotifiedFlag) && (CurrentState & MaybeWaitingFlag))
		{
			ParkingLot::WakeAll(&State);
		}
	}

	/**
	 * Resets the event to a non-notified state.
	 */
	inline void Reset()
	{
		State.fetch_and(~IsNotifiedFlag, std::memory_order_release);
	}

private:
	static constexpr uint8 IsNotifiedFlag = 1 << 0;
	static constexpr uint8 MaybeWaitingFlag = 1 << 1;

	std::atomic<uint8> State = 0;
};

} // UE
