// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Stats/Stats.h"


DECLARE_MEMORY_STAT_EXTERN(TEXT("Mapped File Handle Memory"), STAT_MappedFileMemory, STATGROUP_Memory, CORE_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Mapped File Handles"), STAT_MappedFileHandles, STATGROUP_Memory, CORE_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Mapped File Regions"), STAT_MappedFileRegions, STATGROUP_Memory, CORE_API);

// Note on threading. Like the rest of the filesystem platform abstraction, these methods are threadsafe, but it is expected you are not concurrently _using_ these data structures. 

class IMappedFileRegion
{
	const uint8* MappedPtr;
	size_t MappedSize;
	FString DebugFilename;
	size_t DebugOffsetRelativeToFile;

	UE_FORCEINLINE_HINT void CheckInvariants()
	{
		check(MappedPtr && MappedSize);
	}

public:

	inline IMappedFileRegion(const uint8* InMappedPtr, size_t InMappedSize, const FString& InDebugFilename, size_t InDebugOffsetRelativeToFile)
		: MappedPtr(InMappedPtr)
		, MappedSize(InMappedSize)
		, DebugFilename(InDebugFilename)
		, DebugOffsetRelativeToFile(InDebugOffsetRelativeToFile)
	{
		CheckInvariants();
		INC_DWORD_STAT(STAT_MappedFileRegions);
		INC_MEMORY_STAT_BY(STAT_MappedFileMemory, MappedSize);
	}

	virtual ~IMappedFileRegion()
	{
		check(MappedSize);  
		DEC_MEMORY_STAT_BY(STAT_MappedFileMemory, MappedSize);
		DEC_DWORD_STAT(STAT_MappedFileRegions);
	}


	/**
	* Return the pointer to the mapped region. 
	* @return Returned size of the file or -1 if the file was not found, the request was canceled or other error.
	**/
	inline const uint8* GetMappedPtr()
	{
		CheckInvariants();
		return MappedPtr;
	}

	/**
	* Return the size of the mapped region.
	**/
	inline int64 GetMappedSize()
	{
		CheckInvariants();
		return MappedSize;
	}

	/**
	* Synchronously preload part or all of the mapped region. Typically this is done by reading a byte from each CPU page.
	* This is only a hint, some platforms might ignore it.
	* There are not guarantees how long this data will stay in memory.
	* @param PreloadOffset		Offset into this region to preload
	* @param BytesToPreload		number of bytes to preload. This is clamped to the size of the mapped region
	**/
	virtual void PreloadHint(int64 PreloadOffset = 0, int64 BytesToPreload = MAX_int64)
	{
	}

	/**
	* Synchronously flush part or all of the mapped region.
	* This is only a hint, some platforms might ignore it.
	* There are not guarantees how long this data will stay in paged out.
	* @param FlushOffset		Offset into this region to flush
	* @param BytesToFlush		number of bytes to flush. This is clamped to the size of the mapped region
	**/
	virtual void Flush(int64 FlushOffset = 0, int64 BytesToFlush = MAX_int64)
	{
	}

	// Non-copyable
	IMappedFileRegion(const IMappedFileRegion&) = delete;
	IMappedFileRegion& operator=(const IMappedFileRegion&) = delete;
};

enum class EMappedFileFlags
{
	ENone = 0,				// Do nothing
	EPreloadHint = 1,		// Preload the data. This is only a hint and might be ignored, see IMappedFileRegion::PreloadHint
	EFileWritable = 2,		// Make the mapped file writable. Requires OpenMappedEx2 to be called with OpenOptions set to EOpenMappedFlags::OpenForWrite.
							// This will create a shared mapping on Unix platform to allow any writes to be flushed to disk
};

ENUM_CLASS_FLAGS(EMappedFileFlags);


struct FFileMappingFlags
{
	FFileMappingFlags() : Flags(EMappedFileFlags::ENone) {}
	FFileMappingFlags(bool bPreloadHint) : Flags(bPreloadHint ? EMappedFileFlags::EPreloadHint : EMappedFileFlags::ENone) {}
	FFileMappingFlags(EMappedFileFlags InFlags) : Flags(InFlags) {}
	EMappedFileFlags Flags;
};

class IMappedFileHandle
{
	size_t MappedFileSize;

public:
	IMappedFileHandle(size_t InFileSize)
		: MappedFileSize(InFileSize)
	{
		INC_DWORD_STAT(STAT_MappedFileHandles);
	}
	/** Destructor, also the only way to close the file handle. It is not legal to delete an async file with outstanding requests. You must always call WaitCompletion before deleting a request. **/
	virtual ~IMappedFileHandle()
	{
		DEC_DWORD_STAT(STAT_MappedFileHandles);
	}

	/**
	* Return the size of the mapped file.
	**/
	UE_FORCEINLINE_HINT int64 GetFileSize() const
	{
		return MappedFileSize;
	}

	/**
	* Map a region of the file.
	* @param Offset				Offset into the file to start mapping.
	* @param BytesToMap			Number of bytes to map. Clamped to the size of the file.
	* @param Flags				A combination of EMappingFlags values
	* @return the mapped region interface. Returns nullptr on failure.
	**/
	virtual IMappedFileRegion* MapRegion(int64 Offset = 0, int64 BytesToMap = MAX_int64, FFileMappingFlags Flags = EMappedFileFlags::ENone) = 0;

	virtual void Flush(void)
	{
	}

	// Non-copyable
	IMappedFileHandle(const IMappedFileHandle&) = delete;
	IMappedFileHandle& operator=(const IMappedFileHandle&) = delete;
};
