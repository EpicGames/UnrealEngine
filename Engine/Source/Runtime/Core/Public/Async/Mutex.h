// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/LockTags.h"
#include "AutoRTFM/ReportHazard.h"
#include "CoreTypes.h"
#include <atomic>

#define UE_API CORE_API

namespace UE
{

/**
 * A one-byte mutex that is not fair and does not support recursive locking.
 */
class FMutex final
{
public:
	constexpr FMutex() = default;

	/** Construct in a locked state. Avoids an expensive compare-and-swap at creation time. */
	inline explicit FMutex(FAcquireLock)
		: State(IsLockedFlag)
	{
	}

	FMutex(const FMutex&) = delete;
	FMutex& operator=(const FMutex&) = delete;

	[[nodiscard]] inline bool IsLocked() const
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(Mutex);
		return !!(State.load(std::memory_order_relaxed) & IsLockedFlag);
	}

	[[nodiscard]] inline bool TryLock()
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(Mutex);
		uint8 Expected = State.load(std::memory_order_relaxed);
		return !(Expected & IsLockedFlag) &&
			State.compare_exchange_strong(Expected, Expected | IsLockedFlag, std::memory_order_acquire, std::memory_order_relaxed);
	}

	inline void Lock()
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(Mutex);
		uint8 Expected = 0;
		if (LIKELY(State.compare_exchange_weak(Expected, IsLockedFlag, std::memory_order_acquire, std::memory_order_relaxed)))
		{
			return;
		}
		LockSlow();
	}

	inline void Unlock()
	{
		// Unlock immediately to allow other threads to acquire the lock while this thread looks for a thread to wake.
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(Mutex);
		const uint8 LastState = State.fetch_sub(IsLockedFlag, std::memory_order_release);
		if (LIKELY(!(LastState & MayHaveWaitingLockFlag)))
		{
			return;
		}
		WakeWaitingThread();
	}

private:
	UE_API void LockSlow();
	UE_API void WakeWaitingThread();

	struct FParams;

	static constexpr uint8 IsLockedFlag = 1 << 0;
	static constexpr uint8 MayHaveWaitingLockFlag = 1 << 1;

	std::atomic<uint8> State = 0;
};

} // UE

#undef UE_API
