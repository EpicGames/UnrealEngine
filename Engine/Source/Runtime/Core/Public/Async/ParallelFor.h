// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParllelFor.h: TaskGraph library
=============================================================================*/

#pragma once

#include "AutoRTFM.h"
#include "Async/Fundamental/Scheduler.h"
#include "Async/Fundamental/Task.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "HAL/Event.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformTime.h"
#include "HAL/ThreadSafeCounter.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/App.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Fork.h"
#include "Misc/MemStack.h"
#include "Misc/Timespan.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/TagTrace.h"
#include "Stats/Stats.h"
#include "Templates/Function.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

#include <atomic>

namespace UE { namespace LLMPrivate { class FTagData; } }

extern CORE_API int32 GParallelForBackgroundYieldingTimeoutMs;
extern CORE_API bool GParallelForDisableOversubscription;

// Flags controlling the ParallelFor's behavior.
enum class EParallelForFlags
{
	// Default behavior
	None,

	//Mostly used for testing, when used, ParallelFor will run single threaded instead.
	ForceSingleThread = 1,

	//Offers better work distribution among threads at the cost of a little bit more synchronization.
	//This should be used for tasks with highly variable computational time.
	Unbalanced = 2,

	// if running on the rendering thread, make sure the ProcessThread is called when idle
	PumpRenderingThread = 4,

	// tasks should run on background priority threads
	BackgroundPriority = 8,
};

ENUM_CLASS_FLAGS(EParallelForFlags)

namespace ParallelForImpl
{

	// Helper to call body with context reference
	template <typename FunctionType, typename ContextType>
	inline void CallBody(const FunctionType& Body, const TArrayView<ContextType>& Contexts, int32 TaskIndex, int32 Index)
	{
		Body(Contexts[TaskIndex], Index);
	}

	// Helper specialization for "no context", which changes the assumed body call signature
	template <typename FunctionType>
	inline void CallBody(const FunctionType& Body, const TArrayView<TYPE_OF_NULLPTR>&, int32, int32 Index)
	{
		Body(Index);
	}

	inline int32 GetNumberOfThreadTasks(int32 Num, int32 MinBatchSize, EParallelForFlags Flags)
	{
		int32 NumThreadTasks = 0;
		const bool bIsMultithread = FApp::ShouldUseThreadingForPerformance() || FForkProcessHelper::IsForkedMultithreadInstance();
		if (Num > 1 && (Flags & EParallelForFlags::ForceSingleThread) == EParallelForFlags::None && bIsMultithread)
		{
			NumThreadTasks = FMath::Min(int32(LowLevelTasks::FScheduler::Get().GetNumWorkers()), (Num + (MinBatchSize/2))/MinBatchSize);
		}

		if (!LowLevelTasks::FScheduler::Get().IsWorkerThread())
		{
			NumThreadTasks++; //named threads help with the work
		}

		// don't go wider than number of cores
		NumThreadTasks = FMath::Min(NumThreadTasks, FPlatformMisc::NumberOfCoresIncludingHyperthreads());

		return FMath::Max(NumThreadTasks, 1);
	}

	/** 
		*	General purpose parallel for that uses the taskgraph
		*	@param DebugName; Debugname and Profiling TraceTag
		*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
		*	@param MinBatchSize; Minimum size a Batch should have
		*	@param Body; Function to call from multiple threads
		*	@param CurrentThreadWorkToDoBeforeHelping; The work is performed on the main thread before it starts helping with the ParallelFor proper
		*	@param Flags; Used to customize the behavior of the ParallelFor if needed.
		*   @param Contexts; Optional per thread contexts to accumulate data concurrently.
		*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
	**/
	template<typename BodyType, typename PreWorkType, typename ContextType>
	inline void ParallelForInternal(const TCHAR* DebugName, int32 Num, int32 MinBatchSize, BodyType Body, PreWorkType CurrentThreadWorkToDoBeforeHelping, EParallelForFlags Flags, const TArrayView<ContextType>& Contexts)
	{
		if (Num == 0)
		{
			// Contract is that prework should always be called even when number of tasks is 0.
			// We omit the trace scope here to avoid noise when the prework is empty since this amounts to just calling a function anyway with nothing specific to parallelfor itself.
			CurrentThreadWorkToDoBeforeHelping();
			return;
		}

		SCOPE_CYCLE_COUNTER(STAT_ParallelFor);
		TRACE_CPUPROFILER_EVENT_SCOPE(ParallelFor);
		check(Num >= 0);

		// If we are executing the parallel for from within a transaction, run it single-threaded.
		if (AutoRTFM::IsClosed())
		{
			Flags = EParallelForFlags::ForceSingleThread;
		}

		int32 NumWorkers = GetNumberOfThreadTasks(Num, MinBatchSize, Flags);

		if (!Contexts.IsEmpty())
		{
			// Use at most as many workers as there are contexts when task contexts are used.
			NumWorkers = FMath::Min(NumWorkers, Contexts.Num());
		}

		//single threaded mode
		if (NumWorkers <= 1)
		{
			// do the prework
			CurrentThreadWorkToDoBeforeHelping();
			// no threads, just do it and return
			for(int32 Index = 0; Index < Num; Index++)
			{
				CallBody(Body, Contexts, 0, Index);
			}
			return;
		}
	
		//calculate the batch sizes
		int32 BatchSize = 1;
		int32 NumBatches = Num;
		bool bIsUnbalanced = (Flags & EParallelForFlags::Unbalanced) == EParallelForFlags::Unbalanced;
		if (!bIsUnbalanced)
		{
			for (int32 Div = 6; Div; Div--)
			{
				if (Num >= (NumWorkers * Div))
				{	
					BatchSize = FMath::DivideAndRoundUp<int32>(Num, (NumWorkers * Div));
					NumBatches = FMath::DivideAndRoundUp<int32>(Num, BatchSize);

					if (NumBatches >= NumWorkers)
					{
						break;
					}
				}
			}
		}
		NumWorkers--; //Decrement one because this function will work on it locally
		checkSlow(BatchSize * NumBatches >= Num);

		//Try to inherit the Priority from the caller
		// Anything scheduled by the task graph is latency sensitive because it might impact the frame rate. Anything else is not (i.e. Worker / Background threads).
		const ETaskTag LatencySensitiveTasks = 
			ETaskTag::EStaticInit | 
			ETaskTag::EGameThread | 
			ETaskTag::ESlateThread | 
			ETaskTag::ERenderingThread | 
			ETaskTag::ERhiThread;

		const bool bBackgroundPriority = (Flags & EParallelForFlags::BackgroundPriority) != EParallelForFlags::None;
		const bool bIsLatencySensitive = (FTaskTagScope::GetCurrentTag() & LatencySensitiveTasks) != ETaskTag::ENone;

		LowLevelTasks::ETaskPriority Priority = LowLevelTasks::ETaskPriority::Inherit;
		if (bIsLatencySensitive && !bBackgroundPriority)
		{
			Priority =  LowLevelTasks::ETaskPriority::High;
		}
		else if (bBackgroundPriority)
		{
			Priority = LowLevelTasks::ETaskPriority::BackgroundNormal;
		}
		
		struct FTracedTask
		{
			LowLevelTasks::FTask Task;
			std::atomic<TaskTrace::FId> TraceId = TaskTrace::InvalidId;
		};

		//shared data between tasks
		struct alignas(PLATFORM_CACHE_LINE_SIZE) FParallelForData 
			: public TConcurrentLinearObject<FParallelForData, FTaskGraphBlockAllocationTag>
			, public FRefCountedObject
			, private UE::FInheritedContextBase
		{
			using UE::FInheritedContextBase::RestoreInheritedContext;

			FParallelForData(const TCHAR* InDebugName, int32 InNum, int32 InBatchSize, int32 InNumBatches, int32 InNumWorkers, const TArrayView<ContextType>& InContexts, const BodyType& InBody, FEventRef& InFinishedSignal, LowLevelTasks::ETaskPriority InPriority)
				: DebugName(InDebugName)
				, Num(InNum)
				, BatchSize(InBatchSize)
				, NumBatches(InNumBatches)
				, Contexts(InContexts)
				, Body(InBody)
				, FinishedSignal(InFinishedSignal)
				, Priority(InPriority)
			{
				IncompleteBatches.store(NumBatches, std::memory_order_relaxed);
				Tasks.AddDefaulted(InNumWorkers);

				CaptureInheritedContext();
			}

			~FParallelForData()
			{
				for (FTracedTask& Task : Tasks)
				{
					if (Task.TraceId != TaskTrace::InvalidId)
					{
						TaskTrace::Destroyed(Task.TraceId);
					}
				}
			}

			int32 GetNextWorkerIndexToLaunch()
			{
				const int32 WorkerIndex = LaunchedWorkers.fetch_add(1, std::memory_order_relaxed);
				return WorkerIndex >= Tasks.Num() ? -1 : WorkerIndex;
			}

			const TCHAR* DebugName;
			std::atomic_int BatchItem  { 0 };
			std::atomic_int IncompleteBatches { 0 };
			std::atomic_int LaunchedWorkers { 0 };
			int32 Num;
			int32 BatchSize;
			int32 NumBatches;
			const TArrayView<ContextType>& Contexts;
			const BodyType& Body;
			FEventRef& FinishedSignal;
			LowLevelTasks::ETaskPriority Priority;

			TArray<FTracedTask, TConcurrentLinearArrayAllocator<FTaskGraphBlockAllocationTag>> Tasks;
		};
		using FDataHandle = TRefCountPtr<FParallelForData>;

		// Each task has an executor.
		class FParallelExecutor
		{
			mutable FDataHandle Data;
			int32 WorkerIndex;
			mutable bool bReschedule = false;

		public:
			inline FParallelExecutor(FDataHandle&& InData, int32 InWorkerIndex)
				: Data(MoveTemp(InData))
				, WorkerIndex(InWorkerIndex)
			{
			}

			FParallelExecutor(const FParallelExecutor&) = delete;
			FParallelExecutor(FParallelExecutor&& Other) = default;

			~FParallelExecutor()
			{
				if (Data.IsValid() && bReschedule)
				{
					FParallelExecutor::LaunchTask(MoveTemp(Data), WorkerIndex);
				}
			}

			inline const FDataHandle& GetData() const
			{
				return Data;
			}

			inline bool operator()(const bool bIsMaster = false) const noexcept
			{
				UE::FInheritedContextScope InheritedContextScope = Data->RestoreInheritedContext();
				FMemMark Mark(FMemStack::Get());

				TaskTrace::FId TraceId = TaskTrace::InvalidId;
				if (!bIsMaster)
				{
					TraceId = Data->Tasks[WorkerIndex].TraceId;
					TaskTrace::Started(TraceId);
				}
				ON_SCOPE_EXIT
				{
					if (!bIsMaster)
					{
						TaskTrace::Completed(TraceId);
					}
				};

				const int32 NumBatches = Data->NumBatches;

				// We're going to consume one ourself, so we need at least 2 left to consider launching a new worker
				// We also do not launch a worker from the master as we already launched one before doing prework.
				if (bIsMaster == false && Data->BatchItem.load(std::memory_order_relaxed) + 2 <= NumBatches)
				{
					LaunchAnotherWorkerIfNeeded(Data);
				}

				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(Data->DebugName);

				auto Now = [] { return FTimespan::FromSeconds(FPlatformTime::Seconds()); };
				FTimespan Start = FTimespan::MinValue();
				FTimespan YieldingThreshold;
				const bool bIsBackgroundPriority = !bIsMaster && (Data->Priority >= LowLevelTasks::ETaskPriority::BackgroundNormal);

				if (bIsBackgroundPriority)
				{
					Start = Now();
					YieldingThreshold = FTimespan::FromMilliseconds(FMath::Max(0, GParallelForBackgroundYieldingTimeoutMs));
				}

				const int32 Num = Data->Num;
				const int32 BatchSize = Data->BatchSize;
				const TArrayView<ContextType>& Contexts = Data->Contexts;
				const BodyType& Body = Data->Body;

				const bool bSaveLastBlockForMaster = (Num > NumBatches);
				for(;;)
				{
					int32 BatchIndex = Data->BatchItem.fetch_add(1, std::memory_order_relaxed);
					
					// Save the last block for the master to avoid an event
					if (bSaveLastBlockForMaster && BatchIndex >= NumBatches - 1)
					{
						if (!bIsMaster)
						{
							return false;
						}
						BatchIndex = (NumBatches - 1);
					}

					int32 StartIndex = BatchIndex * BatchSize;
					int32 EndIndex = FMath::Min<int32>(StartIndex + BatchSize, Num);
					for (int32 Index = StartIndex; Index < EndIndex; Index++)
					{
						CallBody(Body, Contexts, WorkerIndex, Index);
					}

					// We need to decrement IncompleteBatches when processing a Batch because we need to know if we are the last one
					// so that if the main thread is the last one we can avoid an FEvent call.

					// Memory ordering is also very important here as it is what's making sure memory manipulated
					// by the parallelfor is properly published before exiting so that it's safe to be read
					// without other synchronization mechanism.
					if (StartIndex < Num && Data->IncompleteBatches.fetch_sub(1, std::memory_order_acq_rel) == 1)
					{
						if (!bIsMaster)
						{
							Data->FinishedSignal->Trigger();
						}

						return true;
					}
					else if (EndIndex >= Num)
					{
						return false;
					}
					else if (!bIsBackgroundPriority)
					{
						continue;
					}

					auto PassedTime = [Start, &Now]() { return Now() - Start; };
					if (PassedTime() > YieldingThreshold)
					{
						// Abort and reschedule to give higher priority tasks a chance to run
						bReschedule = true;
						return false;
					}
				}
			}

			static void LaunchTask(FDataHandle&& InData, int32 InWorkerIndex, bool bWakeUpWorker = true)
			{
				FTracedTask& TracedTask = InData->Tasks[InWorkerIndex];

				if (TracedTask.TraceId != TaskTrace::InvalidId) // reused task
				{
					TaskTrace::Destroyed(TracedTask.TraceId);
				}

				const TCHAR* DebugName = InData->DebugName;
				LowLevelTasks::ETaskPriority Priority = InData->Priority;

				TracedTask.TraceId = TaskTrace::GenerateTaskId();
				TaskTrace::Launched(TracedTask.TraceId, DebugName, false, ENamedThreads::AnyThread, 0);

				TracedTask.Task.Init(DebugName, Priority, FParallelExecutor(MoveTemp(InData), InWorkerIndex));
				verify(LowLevelTasks::TryLaunch(TracedTask.Task, LowLevelTasks::EQueuePreference::GlobalQueuePreference, bWakeUpWorker));
			}

			static void LaunchAnotherWorkerIfNeeded(FDataHandle& InData)
			{
				const int32 WorkerIndex = InData->GetNextWorkerIndexToLaunch();
				if (WorkerIndex != -1)
				{
					LaunchTask(FDataHandle(InData), WorkerIndex);
				}
			}
		};

		//launch all the worker tasks
		FEventRef FinishedSignal { EEventMode::ManualReset };
		FDataHandle Data = new FParallelForData(DebugName, Num, BatchSize, NumBatches, NumWorkers, Contexts, Body, FinishedSignal, Priority);

		// Launch the first worker before we start doing prework
		FParallelExecutor::LaunchAnotherWorkerIfNeeded(Data);

		// do the prework
		CurrentThreadWorkToDoBeforeHelping();

		// help with the parallel-for to prevent deadlocks
		FParallelExecutor LocalExecutor(MoveTemp(Data), NumWorkers);
		const bool bFinishedLast = LocalExecutor(true);

		if (!bFinishedLast)
		{
			const bool bPumpRenderingThread  = (Flags & EParallelForFlags::PumpRenderingThread) != EParallelForFlags::None;
			if (bPumpRenderingThread && IsInActualRenderingThread())
			{
				// FinishedSignal waits here if some other thread finishes the last item
				// Data must live on until all of the tasks are cleared which might be long after this function exits
				while (!FinishedSignal->Wait(1))
				{
					FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GetRenderThread_Local());
				}
			}
			else
			{
				// FinishedSignal waits here if some other thread finishes the last item
				// Data must live on until all of the tasks are cleared which might be long after this function exits
				TRACE_CPUPROFILER_EVENT_SCOPE(ParallelFor.Wait);

				if (GParallelForDisableOversubscription)
				{
					LowLevelTasks::Private::FOversubscriptionAllowedScope Scope(false);
					FinishedSignal->Wait();
				}
				else
				{
					// This can spawn new threads to handle tasks
					FinishedSignal->Wait();
				}
			}
		}
		checkSlow(LocalExecutor.GetData()->BatchItem.load(std::memory_order_relaxed) * LocalExecutor.GetData()->BatchSize >= LocalExecutor.GetData()->Num);
	}
}

/** 
	*	General purpose parallel for that uses the taskgraph
	*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
	*	@param Body; Function to call from multiple threads
	*	@param bForceSingleThread; Mostly used for testing, if true, run single threaded instead.
	*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
**/
inline void ParallelFor(int32 Num, TFunctionRef<void(int32)> Body, bool bForceSingleThread, bool bPumpRenderingThread=false)
{
	ParallelForImpl::ParallelForInternal(TEXT("ParallelFor Task"), Num, 1, Body, [](){},
		(bForceSingleThread ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None) | 
		(bPumpRenderingThread ? EParallelForFlags::PumpRenderingThread : EParallelForFlags::None), TArrayView<TYPE_OF_NULLPTR>());
}

/**
	*	General purpose parallel for that uses the taskgraph
	*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
	*	@param Body; Function to call from multiple threads
	*	@param bForceSingleThread; Mostly used for testing, if true, run single threaded instead.
	*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
**/
template<typename FunctionType>
inline void ParallelForTemplate(int32 Num, const FunctionType& Body, EParallelForFlags Flags = EParallelForFlags::None)
{
	ParallelForImpl::ParallelForInternal(TEXT("ParallelFor Task"), Num, 1, Body, [](){}, Flags, TArrayView<TYPE_OF_NULLPTR>());
}

/**
	*	General purpose parallel for that uses the taskgraph
	*   @param DebugName; ProfilingScope and Debugname
	*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
	*   @param MinBatchSize; Minimum Size of a Batch (will only launch DivUp(Num, MinBatchSize) Workers 
	*	@param Body; Function to call from multiple threads
	*	@param bForceSingleThread; Mostly used for testing, if true, run single threaded instead.
	*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
**/
template<typename FunctionType>
inline void ParallelForTemplate(const TCHAR* DebugName, int32 Num, int32 MinBatchSize, const FunctionType& Body, EParallelForFlags Flags = EParallelForFlags::None)
{
	ParallelForImpl::ParallelForInternal(DebugName, Num, MinBatchSize, Body, [](){}, Flags, TArrayView<TYPE_OF_NULLPTR>());
}

/** 
	*	General purpose parallel for that uses the taskgraph for unbalanced tasks
	*	Offers better work distribution among threads at the cost of a little bit more synchronization.
	*	This should be used for tasks with highly variable computational time.
	*
	*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
	*	@param Body; Function to call from multiple threads
	*	@param Flags; Used to customize the behavior of the ParallelFor if needed.
	*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
**/
inline void ParallelFor(int32 Num, TFunctionRef<void(int32)> Body, EParallelForFlags Flags = EParallelForFlags::None)
{
	ParallelForImpl::ParallelForInternal(TEXT("ParallelFor Task"), Num, 1, Body, [](){}, Flags, TArrayView<TYPE_OF_NULLPTR>());
}

/** 
	*	General purpose parallel for that uses the taskgraph for unbalanced tasks
	*	Offers better work distribution among threads at the cost of a little bit more synchronization.
	*	This should be used for tasks with highly variable computational time.
	*
	*   @param DebugName; ProfilingScope and Debugname
	*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
	*   @param MinBatchSize; Minimum Size of a Batch (will only launch DivUp(Num, MinBatchSize) Workers 
	*	@param Body; Function to call from multiple threads
	*	@param Flags; Used to customize the behavior of the ParallelFor if needed.
	*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
**/
inline void ParallelFor(const TCHAR* DebugName, int32 Num, int32 MinBatchSize, TFunctionRef<void(int32)> Body, EParallelForFlags Flags = EParallelForFlags::None)
{
	ParallelForImpl::ParallelForInternal(DebugName, Num, MinBatchSize, Body, [](){}, Flags, TArrayView<TYPE_OF_NULLPTR>());
}

/** 
	*	General purpose parallel for that uses the taskgraph
	*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
	*	@param Body; Function to call from multiple threads
	*	@param CurrentThreadWorkToDoBeforeHelping; The work is performed on the main thread before it starts helping with the ParallelFor proper
	*	@param bForceSingleThread; Mostly used for testing, if true, run single threaded instead.
	*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
**/
inline void ParallelForWithPreWork(int32 Num, TFunctionRef<void(int32)> Body, TFunctionRef<void()> CurrentThreadWorkToDoBeforeHelping, bool bForceSingleThread, bool bPumpRenderingThread = false)
{
	ParallelForImpl::ParallelForInternal(TEXT("ParallelFor Task"), Num, 1, Body, CurrentThreadWorkToDoBeforeHelping,
		(bForceSingleThread ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None) |
		(bPumpRenderingThread ? EParallelForFlags::PumpRenderingThread : EParallelForFlags::None), TArrayView<TYPE_OF_NULLPTR>());
}

/** 
	*	General purpose parallel for that uses the taskgraph
	*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
	*	@param Body; Function to call from multiple threads
	*	@param CurrentThreadWorkToDoBeforeHelping; The work is performed on the main thread before it starts helping with the ParallelFor proper
	*	@param Flags; Used to customize the behavior of the ParallelFor if needed.
	*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
**/
inline void ParallelForWithPreWork(int32 Num, TFunctionRef<void(int32)> Body, TFunctionRef<void()> CurrentThreadWorkToDoBeforeHelping, EParallelForFlags Flags = EParallelForFlags::None)
{
	ParallelForImpl::ParallelForInternal(TEXT("ParallelFor Task"), Num, 1, Body, CurrentThreadWorkToDoBeforeHelping, Flags, TArrayView<TYPE_OF_NULLPTR>());
}

/** 
	*	General purpose parallel for that uses the taskgraph
	*   @param DebugName; ProfilingScope and Debugname
	*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
	*   @param MinBatchSize; Minimum Size of a Batch (will only launch DivUp(Num, MinBatchSize) Workers 
	*	@param Body; Function to call from multiple threads
	*	@param CurrentThreadWorkToDoBeforeHelping; The work is performed on the main thread before it starts helping with the ParallelFor proper
	*	@param Flags; Used to customize the behavior of the ParallelFor if needed.
	*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
**/
inline void ParallelForWithPreWork(const TCHAR* DebugName, int32 Num, int32 MinBatchSize, TFunctionRef<void(int32)> Body, TFunctionRef<void()> CurrentThreadWorkToDoBeforeHelping, EParallelForFlags Flags = EParallelForFlags::None)
{
	ParallelForImpl::ParallelForInternal(DebugName, Num, MinBatchSize, Body, CurrentThreadWorkToDoBeforeHelping, Flags, TArrayView<TYPE_OF_NULLPTR>());
}

/** 
 * General purpose parallel for that uses the taskgraph
 * @param DebugName; ProfilingScope and DebugName
 * @param OutContexts; Array that will hold the user-defined, task-level context objects (allocated per parallel task)
 * @param Num; number of calls of Body; Body(0), Body(1), ..., Body(Num - 1)
 * @param MinBatchSize; Minimum Size of a Batch (will only launch DivUp(Num, MinBatchSize) Workers 
 * @param ContextConstructor; Function to call to initialize each task context allocated for the operation
 * @param Body; Function to call from multiple threads
 * @param CurrentThreadWorkToDoBeforeHelping; The work is performed on the main thread before it starts helping with the ParallelFor proper
 * @param Flags; Used to customize the behavior of the ParallelFor if needed.
 * Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
 */
template <typename ContextType, typename ContextAllocatorType, typename ContextConstructorType, typename BodyType, typename PreWorkType>
inline void ParallelForWithPreWorkWithTaskContext(
	const TCHAR* DebugName,
	TArray<ContextType, ContextAllocatorType>& OutContexts,
	int32 Num,
	int32 MinBatchSize,
	ContextConstructorType&& ContextConstructor,
	BodyType&& Body,
	PreWorkType&& CurrentThreadWorkToDoBeforeHelping,
	EParallelForFlags Flags = EParallelForFlags::None)
{
	if (Num > 0)
	{
		const int32 NumContexts = ParallelForImpl::GetNumberOfThreadTasks(Num, MinBatchSize, Flags);
		OutContexts.Reset(NumContexts);
		for (int32 ContextIndex = 0; ContextIndex < NumContexts; ++ContextIndex)
		{
			OutContexts.Emplace(ContextConstructor(ContextIndex, NumContexts));
		}
		ParallelForImpl::ParallelForInternal(DebugName, Num, MinBatchSize, Forward<BodyType>(Body), Forward<PreWorkType>(CurrentThreadWorkToDoBeforeHelping), Flags, TArrayView<ContextType>(OutContexts));
	}
}

/** 
 * General purpose parallel for that uses the taskgraph
 * @param DebugName; ProfilingScope and DebugName
 * @param OutContexts; Array that will hold the user-defined, task-level context objects (allocated per parallel task)
 * @param Num; number of calls of Body; Body(0), Body(1), ..., Body(Num - 1)
 * @param MinBatchSize; Minimum Size of a Batch (will only launch DivUp(Num, MinBatchSize) Workers 
 * @param Body; Function to call from multiple threads
 * @param CurrentThreadWorkToDoBeforeHelping; The work is performed on the main thread before it starts helping with the ParallelFor proper
 * @param Flags; Used to customize the behavior of the ParallelFor if needed.
 * Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
 */
template <typename ContextType, typename ContextAllocatorType, typename BodyType, typename PreWorkType>
inline void ParallelForWithPreWorkWithTaskContext(
	const TCHAR* DebugName,
	TArray<ContextType, ContextAllocatorType>& OutContexts,
	int32 Num,
	int32 MinBatchSize,
	BodyType&& Body,
	PreWorkType&& CurrentThreadWorkToDoBeforeHelping,
	EParallelForFlags Flags = EParallelForFlags::None)
{
	if (Num > 0)
	{
		const int32 NumContexts = ParallelForImpl::GetNumberOfThreadTasks(Num, MinBatchSize, Flags);
		OutContexts.Reset();
		OutContexts.AddDefaulted(NumContexts);
		ParallelForImpl::ParallelForInternal(DebugName, Num, MinBatchSize, Forward<BodyType>(Body), Forward<PreWorkType>(CurrentThreadWorkToDoBeforeHelping), Flags, TArrayView<ContextType>(OutContexts));
	}
}

/** 
 * General purpose parallel for that uses the taskgraph
 * @param DebugName; ProfilingScope and DebugName
 * @param Contexts; User-privided array of user-defined task-level context objects
 * @param Num; number of calls of Body; Body(0), Body(1), ..., Body(Num - 1)
 * @param MinBatchSize; Minimum Size of a Batch (will only launch DivUp(Num, MinBatchSize) Workers 
 * @param Body; Function to call from multiple threads
 * @param CurrentThreadWorkToDoBeforeHelping; The work is performed on the main thread before it starts helping with the ParallelFor proper
 * @param Flags; Used to customize the behavior of the ParallelFor if needed.
 * Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
 */
template <typename ContextType, typename BodyType, typename PreWorkType>
inline void ParallelForWithPreWorkWithExistingTaskContext(
	const TCHAR* DebugName,
	TArrayView<ContextType> Contexts,
	int32 Num,
	int32 MinBatchSize,
	BodyType&& Body,
	PreWorkType&& CurrentThreadWorkToDoBeforeHelping,
	EParallelForFlags Flags = EParallelForFlags::None)
{
	ParallelForImpl::ParallelForInternal(DebugName, Num, MinBatchSize, Forward<BodyType>(Body), Forward<PreWorkType>(CurrentThreadWorkToDoBeforeHelping), Flags, Contexts);
}

/** 
	*	General purpose parallel for that uses the taskgraph. This variant constructs for the caller a user-defined context
	* 	object for each task that may get spawned to do work, and passes it on to the loop body to give it a task-local
	*   "workspace" that can be mutated without need for synchronization primitives. For this variant, the user provides a
	* 	callable to construct each context element.
	*	@param OutContexts; Array that will hold the user-defined, task-level context objects (allocated per parallel task)
	*   @param DebugName; ProfilingScope and Debugname
	*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
	* 	@param ContextConstructor; Function to call to initialize each task context allocated for the operation
	*	@param Body; Function to call from multiple threads
	*	@param Flags; Used to customize the behavior of the ParallelFor if needed.
	*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
**/
template <typename ContextType, typename ContextAllocatorType, typename ContextConstructorType, typename FunctionType>
inline void ParallelForWithTaskContext(const TCHAR* DebugName, TArray<ContextType, ContextAllocatorType>& OutContexts, int32 Num, const ContextConstructorType& ContextConstructor, const FunctionType& Body, EParallelForFlags Flags = EParallelForFlags::None)
{
	if (Num > 0)
	{
		const int32 NumContexts = ParallelForImpl::GetNumberOfThreadTasks(Num, 1, Flags);
		OutContexts.Reset();
		OutContexts.AddUninitialized(NumContexts);
		for (int32 ContextIndex = 0; ContextIndex < NumContexts; ++ContextIndex)
		{
			new(&OutContexts[ContextIndex]) ContextType(ContextConstructor(ContextIndex, NumContexts));
		}
		ParallelForImpl::ParallelForInternal(DebugName, Num, 1, Body, [](){}, Flags, TArrayView<ContextType>(OutContexts));
	}
}

/** 
	*	General purpose parallel for that uses the taskgraph. This variant constructs for the caller a user-defined context
	* 	object for each task that may get spawned to do work, and passes it on to the loop body to give it a task-local
	*   "workspace" that can be mutated without need for synchronization primitives. For this variant, the user provides a
	* 	callable to construct each context element.
	*	@param OutContexts; Array that will hold the user-defined, task-level context objects (allocated per parallel task)
	*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
	* 	@param ContextConstructor; Function to call to initialize each task context allocated for the operation
	*	@param Body; Function to call from multiple threads
	*	@param Flags; Used to customize the behavior of the ParallelFor if needed.
	*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
**/
template <typename ContextType, typename ContextAllocatorType, typename ContextConstructorType, typename FunctionType>
inline void ParallelForWithTaskContext(TArray<ContextType, ContextAllocatorType>& OutContexts, int32 Num, const ContextConstructorType& ContextConstructor, const FunctionType& Body, EParallelForFlags Flags = EParallelForFlags::None)
{
	ParallelForWithTaskContext(TEXT("ParallelFor Task"), OutContexts, Num, ContextConstructor, Body, Flags);
}

/** 
	*	General purpose parallel for that uses the taskgraph. This variant constructs for the caller a user-defined context
	* 	object for each task that may get spawned to do work, and passes it on to the loop body to give it a task-local
	*   "workspace" that can be mutated without need for synchronization primitives.
	*	@param OutContexts; Array that will hold the user-defined, task-level context objects (allocated per parallel task)
	*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
	*	@param Body; Function to call from multiple threads
	*	@param Flags; Used to customize the behavior of the ParallelFor if needed.
	*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
**/
template <typename ContextType, typename ContextAllocatorType, typename FunctionType>
inline void ParallelForWithTaskContext(TArray<ContextType, ContextAllocatorType>& OutContexts, int32 Num, const FunctionType& Body, EParallelForFlags Flags = EParallelForFlags::None)
{
	if (Num > 0)
	{
		const int32 NumContexts = ParallelForImpl::GetNumberOfThreadTasks(Num, 1, Flags);
		OutContexts.Reset();
		OutContexts.AddDefaulted(NumContexts);
		ParallelForImpl::ParallelForInternal(TEXT("ParallelFor Task"), Num, 1, Body, [](){}, Flags, TArrayView<ContextType>(OutContexts));
	}
}

/** 
	*	General purpose parallel for that uses the taskgraph. This variant constructs for the caller a user-defined context
	* 	object for each task that may get spawned to do work, and passes it on to the loop body to give it a task-local
	*   "workspace" that can be mutated without need for synchronization primitives. For this variant, the user provides a
	* 	callable to construct each context element.
	*	@param OutContexts; Array that will hold the user-defined, task-level context objects (allocated per parallel task)
	*   @param DebugName; ProfilingScope and Debugname
	*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
	*   @param MinBatchSize; Minimum Size of a Batch (will only launch DivUp(Num, MinBatchSize) Workers 
	* 	@param ContextConstructor; Function to call to initialize each task context allocated for the operation
	*	@param Body; Function to call from multiple threads
	*	@param Flags; Used to customize the behavior of the ParallelFor if needed.
	*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
**/
template <typename ContextType, typename ContextAllocatorType, typename ContextConstructorType, typename FunctionType>
inline void ParallelForWithTaskContext(const TCHAR* DebugName, TArray<ContextType, ContextAllocatorType>& OutContexts, int32 Num, int32 MinBatchSize, const ContextConstructorType& ContextConstructor, const FunctionType& Body, EParallelForFlags Flags = EParallelForFlags::None)
{
	if (Num > 0)
	{
		const int32 NumContexts = ParallelForImpl::GetNumberOfThreadTasks(Num, MinBatchSize, Flags);
		OutContexts.Reset();
		OutContexts.AddUninitialized(NumContexts);
		for (int32 ContextIndex = 0; ContextIndex < NumContexts; ++ContextIndex)
		{
			new(&OutContexts[ContextIndex]) ContextType(ContextConstructor(ContextIndex, NumContexts));
		}
		ParallelForImpl::ParallelForInternal(DebugName, Num, MinBatchSize, Body, [](){}, Flags, TArrayView<ContextType>(OutContexts));
	}
}

/** 
	*	General purpose parallel for that uses the taskgraph. This variant constructs for the caller a user-defined context
	* 	object for each task that may get spawned to do work, and passes it on to the loop body to give it a task-local
	*   "workspace" that can be mutated without need for synchronization primitives.
	*	@param OutContexts; Array that will hold the user-defined, task-level context objects (allocated per parallel task)
	*   @param DebugName; ProfilingScope and Debugname
	*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
	*   @param MinBatchSize; Minimum Size of a Batch (will only launch DivUp(Num, MinBatchSize) Workers 
	*	@param Body; Function to call from multiple threads
	*	@param Flags; Used to customize the behavior of the ParallelFor if needed.
	*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
**/
template <typename ContextType, typename ContextAllocatorType, typename FunctionType>
inline void ParallelForWithTaskContext(const TCHAR* DebugName, TArray<ContextType, ContextAllocatorType>& OutContexts, int32 Num, int32 MinBatchSize, const FunctionType& Body, EParallelForFlags Flags = EParallelForFlags::None)
{
	if (Num > 0)
	{
		const int32 NumContexts = ParallelForImpl::GetNumberOfThreadTasks(Num, MinBatchSize, Flags);
		OutContexts.Reset();
		OutContexts.AddDefaulted(NumContexts);
		ParallelForImpl::ParallelForInternal(DebugName, Num, MinBatchSize, Body, [](){}, Flags, TArrayView<ContextType>(OutContexts));
	}
}

/**
*	General purpose parallel for that uses the taskgraph. This variant takes an array of user-defined context
*	objects for each task that may get spawned to do work (one task per context at most), and passes them to
*	the loop body to give it a task-local "workspace" that can be mutated without need for synchronization primitives.
*	@param Contexts; User-privided array of user-defined task-level context objects
*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
*	@param MinBatchSize; Minimum Size of a Batch (will only launch DivUp(Num, MinBatchSize) Workers 
*	@param Body; Function to call from multiple threads
*	@param Flags; Used to customize the behavior of the ParallelFor if needed.
*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
**/
template <typename ContextType, typename FunctionType>
inline void ParallelForWithExistingTaskContext(TArrayView<ContextType> Contexts, int32 Num, int32 MinBatchSize, const FunctionType& Body, EParallelForFlags Flags = EParallelForFlags::None)
{
	ParallelForImpl::ParallelForInternal(TEXT("ParallelFor Task"), Num, MinBatchSize, Body, [](){}, Flags, Contexts);
}

/**
*	General purpose parallel for that uses the taskgraph. This variant takes an array of user-defined context
*	objects for each task that may get spawned to do work (one task per context at most), and passes them to
*	the loop body to give it a task-local "workspace" that can be mutated without need for synchronization primitives.
*	@param DebugName; ProfilingScope and Debugname
*	@param Contexts; User-privided array of user-defined task-level context objects
*	@param Num; number of calls of Body; Body(0), Body(1)....Body(Num - 1)
*	@param MinBatchSize; Minimum Size of a Batch (will only launch DivUp(Num, MinBatchSize) Workers 
*	@param Body; Function to call from multiple threads
*	@param Flags; Used to customize the behavior of the ParallelFor if needed.
*	Notes: Please add stats around to calls to parallel for and within your lambda as appropriate. Do not clog the task graph with long running tasks or tasks that block.
**/
template <typename ContextType, typename FunctionType>
inline void ParallelForWithExistingTaskContext(const TCHAR* DebugName, TArrayView<ContextType> Contexts, int32 Num, int32 MinBatchSize, const FunctionType& Body, EParallelForFlags Flags = EParallelForFlags::None)
{
	ParallelForImpl::ParallelForInternal(DebugName, Num, MinBatchSize, Body, [](){}, Flags, Contexts);
}
