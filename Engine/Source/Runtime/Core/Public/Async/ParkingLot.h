// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/MonotonicTime.h"
#include "Templates/FunctionWithContext.h"

#include <type_traits>

#define UE_API CORE_API

/**
 * ParkingLot is a global table of queues of waiting threads keyed by memory address.
 */
namespace UE::ParkingLot
{

struct FWaitState final
{
	/** Did this thread wait? True only if CanWait returned true. */
	bool bDidWait = false;
	/** Did this wake from a wait? True only if a Wake call woke the thread, false for timeouts. */
	bool bDidWake = false;
	/** Optional value that was provided by the callback in WakeOne. */
	uint64 WakeToken = 0;
};

struct FWakeState final
{
	/** Did a thread wake up? */
	bool bDidWake = false;
	/** Does the queue MAYBE have another thread waiting on this address? */
	bool bHasWaitingThreads = false;
};

namespace Private
{
UE_API FWaitState Wait(const void* Address, bool (*CanWait)(void*), void* CanWaitContext, void (*BeforeWait)(void*), void* BeforeWaitContext);
UE_API FWaitState WaitFor(const void* Address, bool (*CanWait)(void*), void* CanWaitContext, void (*BeforeWait)(void*), void* BeforeWaitContext, FMonotonicTimeSpan WaitTime);
UE_API FWaitState WaitUntil(const void* Address, bool (*CanWait)(void*), void* CanWaitContext, void (*BeforeWait)(void*), void* BeforeWaitContext, FMonotonicTimePoint WaitTime);
UE_API void WakeOne(const void* Address, uint64 (*OnWakeState)(void*, FWakeState), void* OnWakeStateContext);
} // Private

/**
 * Queue the calling thread to wait if CanWait returns true. BeforeWait is only called if CanWait returns true.
 *
 * CanWait is called while the queue for the address is locked. It is invalid to call into any ParkingLot function from
 * within a CanWait function. No other thread can wake or start waiting on this address while within CanWait. Returning
 * false from CanWait will return from Wait without waiting.
 *
 * BeforeWait is called only when CanWait returns true. It is called after the queue for the address has been unlocked.
 * It is invalid to call into any ParkingLot wait function from within BeforeWait, but it is safe to call into any wake
 * function and may be necessary as part of the implementation of some synchronization primitives.
 *
 * @param Address      Address to use as the key for the queue. The same address is used to wake the thread.
 * @param CanWait      Function called while the queue is locked. A return of false cancels the wait.
 * @param BeforeWait   Function called after the queue is unlocked and before the thread waits.
 */
inline FWaitState Wait(const void* Address, TFunctionWithContext<bool()> CanWait, TFunctionWithContext<void()> BeforeWait)
{
	return Private::Wait(Address, CanWait.GetFunction(), CanWait.GetContext(), BeforeWait.GetFunction(), BeforeWait.GetContext());
}

/**
 * Queue the calling thread to wait if CanWait returns true. BeforeWait is only called if CanWait returns true.
 *
 * @param Address      Address to use as the key for the queue. The same address is used to wake the thread.
 * @param CanWait      Function called while the queue is locked. A return of false cancels the wait.
 * @param BeforeWait   Function called after the queue is unlocked and before the thread waits.
 * @param WaitTime     Relative time after which waiting is automatically canceled and the thread wakes.
 */
inline FWaitState WaitFor(const void* Address, TFunctionWithContext<bool()> CanWait, TFunctionWithContext<void()> BeforeWait, FMonotonicTimeSpan WaitTime)
{
	return Private::WaitFor(Address, CanWait.GetFunction(), CanWait.GetContext(), BeforeWait.GetFunction(), BeforeWait.GetContext(), WaitTime);
}

/**
 * Queue the calling thread to wait if CanWait returns true. BeforeWait is only called if CanWait returns true.
 *
 * @param Address      Address to use as the key for the queue. The same address is used to wake the thread.
 * @param CanWait      Function called while the queue is locked. A return of false cancels the wait.
 * @param BeforeWait   Function called after the queue is unlocked and before the thread waits.
 * @param WaitTime     Absolute time after which waiting is automatically canceled and the thread wakes.
 */
inline FWaitState WaitUntil(const void* Address, TFunctionWithContext<bool()> CanWait, TFunctionWithContext<void()> BeforeWait, FMonotonicTimePoint WaitTime)
{
	return Private::WaitUntil(Address, CanWait.GetFunction(), CanWait.GetContext(), BeforeWait.GetFunction(), BeforeWait.GetContext(), WaitTime);
}

/**
 * Wake one thread from the queue of threads waiting on the address.
 *
 * @param Address       Address to use as the key for the queue. Must match the address used in Wait.
 * @param OnWakeState   Function called while the queue is locked. Receives the wake state. Returns WakeToken.
 */
inline void WakeOne(const void* Address, TFunctionWithContext<uint64(FWakeState)> OnWakeState)
{
	return Private::WakeOne(Address, OnWakeState.GetFunction(), OnWakeState.GetContext());
}

/**
 * Wake one thread from the queue of threads waiting on the address.
 *
 * @param Address   Address to use as the key for the queue. Must match the address used in Wait.
 * @return The wake state, which includes whether a thread woke up and whether there are more queued.
 */
UE_API FWakeState WakeOne(const void* Address);

/**
 * Wake up to WakeCount threads from the queue of threads waiting on the address.
 *
 * @param Address     Address to use as the key for the queue. Must match the address used in Wait.
 * @param WakeCount   The maximum number of threads to wake.
 * @return The number of threads that this call woke up.
 */
UE_API uint32 WakeMultiple(const void* Address, uint32 WakeCount);

/**
 * Wake all threads from the queue of threads waiting on the address.
 *
 * @param Address   Address to use as the key for the queue. Must match the address used in Wait.
 * @return The number of threads that this call woke up.
 */
UE_API uint32 WakeAll(const void* Address);

} // UE::ParkingLot

#undef UE_API

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Templates/Function.h"
#endif
