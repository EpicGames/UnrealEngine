// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "AutoRTFM/ReportHazard.h"
#include <atomic>

#define UE_API CORE_API

namespace UE
{

/**
 * An eight-byte mutex that is not fair and supports recursive locking.
 *
 * Prefer FMutex when recursive locking is not required.
 */
class FRecursiveMutex final
{
public:
	constexpr FRecursiveMutex() = default;

	FRecursiveMutex(const FRecursiveMutex&) = delete;
	FRecursiveMutex& operator=(const FRecursiveMutex&) = delete;

	[[nodiscard]] inline bool IsLocked() const
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(RecursiveMutex);
		return !!(State.load(std::memory_order_relaxed) & LockCountMask);
	}

	[[nodiscard]] UE_API bool IsLockedByCurrentThread() const;
	[[nodiscard]] UE_API bool TryLock();
	UE_API void Lock();
	UE_API void Unlock();

private:
	void LockSlow(uint32 CurrentState, uint32 CurrentThreadId);
	void WakeWaitingThread();

	static constexpr uint32 MayHaveWaitingLockFlag = 1 << 0;
	static constexpr uint32 LockCountShift = 1;
	static constexpr uint32 LockCountMask = 0xffff'fffe;

	std::atomic<uint32> State = 0;
	std::atomic<uint32> ThreadId = 0;
};

} // UE

#undef UE_API
