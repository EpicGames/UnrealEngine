// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/WordMutex.h"
#include <atomic>

#define UE_API CORE_API

namespace UE
{

/**
 * A recursive mutex that is the size of two pointers and does not depend on ParkingLot.
 *
 * Prefer FRecursiveMutex to FRecursiveWordMutex whenever possible.
 * This mutex is not fair and supports recursive locking.
 *
 * This type is valuable when a mutex must be trivially constructible, trivially
 * destructible, or must be functional before or after static initialization.
 */
class FRecursiveWordMutex final
{
public:
	[[nodiscard]] inline bool IsLocked() const
	{
		return Mutex.IsLocked();
	}

	[[nodiscard]] UE_API bool IsLockedByCurrentThread() const;
	[[nodiscard]] UE_API bool TryLock();
	UE_API void Lock();
	UE_API void Unlock();

private:
	FWordMutex Mutex;
	uint32 RecursionCount = 0;
	std::atomic<uint32> ThreadId = 0;
};

} // UE

#undef UE_API
