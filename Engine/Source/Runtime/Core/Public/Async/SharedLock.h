// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/LockTags.h"
#include "Misc/AssertionMacros.h"

namespace UE
{

// Note: FSharedRecursiveMutex has specialized these lock types.

/**
 * A basic shared mutex ownership wrapper that locks on construction and unlocks on destruction.
 *
 * LockType must contain LockShared() and UnlockShared() functions.
 * 
 * Use with mutex types like FSharedMutex and FSharedRecursiveMutex.
 */
template <typename LockType>
class TSharedLock final
{
public:
	TSharedLock(const TSharedLock&) = delete;
	TSharedLock& operator=(const TSharedLock&) = delete;

	[[nodiscard]] inline explicit TSharedLock(LockType& Lock)
		: Mutex(Lock)
	{
		Mutex.LockShared();
	}

	inline ~TSharedLock()
	{
		Mutex.UnlockShared();
	}

private:
	LockType& Mutex;
};

/**
 * A shared mutex ownership wrapper that allows dynamic locking, unlocking, and deferred locking.
 *
 * LockType must contain LockShared() and UnlockShared() functions.
 * 
 * Use with mutex types like FSharedMutex and FSharedRecursiveMutex.
 */
template <typename LockType>
class TDynamicSharedLock final
{
public:
	TDynamicSharedLock() = default;

	TDynamicSharedLock(const TDynamicSharedLock&) = delete;
	TDynamicSharedLock& operator=(const TDynamicSharedLock&) = delete;

	/** Wrap a mutex and lock it in shared mode. */
	[[nodiscard]] inline explicit TDynamicSharedLock(LockType& Lock)
		: Mutex(&Lock)
	{
		Mutex->LockShared();
		bLocked = true;
	}

	/** Wrap a mutex without locking it in shared mode. */
	[[nodiscard]] inline explicit TDynamicSharedLock(LockType& Lock, FDeferLock)
		: Mutex(&Lock)
	{
	}

	/** Move from another lock, transferring any ownership to this lock. */
	[[nodiscard]] inline TDynamicSharedLock(TDynamicSharedLock&& Other)
		: Mutex(Other.Mutex)
		, bLocked(Other.bLocked)
	{
		Other.Mutex = nullptr;
		Other.bLocked = false;
	}

	/** Move from another lock, transferring any ownership to this lock, and unlocking the previous mutex if locked. */
	inline TDynamicSharedLock& operator=(TDynamicSharedLock&& Other)
	{
		if (bLocked)
		{
			Mutex->UnlockShared();
		}
		Mutex = Other.Mutex;
		bLocked = Other.bLocked;
		Other.Mutex = nullptr;
		Other.bLocked = false;
		return *this;
	}

	/** Unlock the mutex if locked. */
	inline ~TDynamicSharedLock()
	{
		if (bLocked)
		{
			Mutex->UnlockShared();
		}
	}

	/** Try to lock the associated mutex in shared mode. This lock must have a mutex and must not be locked. */
	bool TryLock()
	{
		check(!bLocked);
		check(Mutex);
		bLocked = Mutex->TryLockShared();
		return bLocked;
	}

	/** Lock the associated mutex in shared mode. This lock must have a mutex and must not be locked. */
	void Lock()
	{
		check(!bLocked);
		check(Mutex);
		Mutex->LockShared();
		bLocked = true;
	}

	/** Unlock the associated mutex in shared mode. This lock must have a mutex and must be locked. */
	void Unlock()
	{
		check(bLocked);
		bLocked = false;
		Mutex->UnlockShared();
	}

	/** Returns true if this lock has its associated mutex locked. */
	inline bool OwnsLock() const
	{
		return bLocked;
	}

	/** Returns true if this lock has its associated mutex locked. */
	inline explicit operator bool() const
	{
		return OwnsLock();
	}

private:
	LockType* Mutex = nullptr;
	bool bLocked = false;
};

} // UE
