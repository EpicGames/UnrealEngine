// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "AutoRTFM/ReportHazard.h"
#include "Misc/AssertionMacros.h"
#include <atomic>

#define UE_API CORE_API

namespace UE
{

/**
 * A four-byte shared mutex that is not fair and does not support recursive locking.
 *
 * Prefer FMutex when shared locking is not required.
 * All new shared locks will wait when any thread is waiting to take an exclusive lock.
 * An exclusive lock and a shared lock may not be simultaneously held by the same thread.
 */
class FSharedMutex final
{
public:
	constexpr FSharedMutex() = default;

	FSharedMutex(const FSharedMutex&) = delete;
	FSharedMutex& operator=(const FSharedMutex&) = delete;

	[[nodiscard]] inline bool IsLocked() const
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(SharedMutex);
		return !!(State.load(std::memory_order_relaxed) & IsLockedFlag);
	}

	[[nodiscard]] inline bool TryLock()
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(SharedMutex);
		uint32 Expected = State.load(std::memory_order_relaxed);
		return !(Expected & (IsLockedFlag | SharedLockCountMask)) &&
			State.compare_exchange_strong(Expected, Expected | IsLockedFlag,
				std::memory_order_acquire, std::memory_order_relaxed);
	}

	inline void Lock()
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(SharedMutex);
		uint32 Expected = 0;
		if (LIKELY(State.compare_exchange_weak(Expected, IsLockedFlag, std::memory_order_acquire, std::memory_order_relaxed)))
		{
			return;
		}
		LockSlow();
	}

	inline void Unlock()
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(SharedMutex);
		// Unlock immediately to allow other threads to acquire the lock while this thread looks for a thread to wake.
		uint32 LastState = State.fetch_sub(IsLockedFlag, std::memory_order_release);
		checkSlow(LastState & IsLockedFlag);
		if (LIKELY(!(LastState & (MayHaveWaitingLockFlag | MayHaveWaitingSharedLockFlag))))
		{
			return;
		}
		WakeWaitingThreads(LastState);
	}

	[[nodiscard]] inline bool IsLockShared() const
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(SharedMutex);
		return !!(State.load(std::memory_order_relaxed) & SharedLockCountMask);
	}

	[[nodiscard]] inline bool TryLockShared()
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(SharedMutex);
		uint32 Expected = State.load(std::memory_order_relaxed);
		while (LIKELY(!(Expected & (IsLockedFlag | MayHaveWaitingLockFlag))))
		{
			if (LIKELY(State.compare_exchange_weak(Expected, Expected + (1 << SharedLockCountShift),
					std::memory_order_acquire, std::memory_order_relaxed)))
			{
				return true;
			}
		}
		return false;
	}

	inline void LockShared()
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(SharedMutex);
		uint32 Expected = State.load(std::memory_order_relaxed);
		if (LIKELY(!(Expected & (IsLockedFlag | MayHaveWaitingLockFlag)) &&
			State.compare_exchange_weak(Expected, Expected + (1 << SharedLockCountShift),
				std::memory_order_acquire, std::memory_order_relaxed)))
		{
			return;
		}
		LockSharedSlow();
	}

	inline void UnlockShared()
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(SharedMutex);
		// Unlock immediately to allow other threads to acquire the lock while this thread looks for a thread to wake.
		// C++ SharedMutex requires that UnlockShared() synchronizes-with subsequent Lock() operations.
		// This requirement necessitates the use of release memory order or stronger.
		const uint32 LastState = State.fetch_sub(1 << SharedLockCountShift, std::memory_order_release);
		checkSlow(LastState & SharedLockCountMask);
		constexpr uint32 WakeState = MayHaveWaitingLockFlag | (1 << SharedLockCountShift);
		if (LIKELY((LastState & ~MayHaveWaitingSharedLockFlag) != WakeState))
		{
			return;
		}
		WakeWaitingThread();
	}

private:
	UE_API void LockSlow();
	UE_API void LockSharedSlow();
	UE_API void WakeWaitingThread();
	UE_API void WakeWaitingThreads(uint32 LastState);

	const void* GetSharedLockAddress() const;

	struct FParams;

	static constexpr uint32 IsLockedFlag = 1 << 0;
	static constexpr uint32 MayHaveWaitingLockFlag = 1 << 1;
	static constexpr uint32 MayHaveWaitingSharedLockFlag = 1 << 2;
	static constexpr uint32 SharedLockCountShift = 3;
	static constexpr uint32 SharedLockCountMask = 0xffff'fff8;

	std::atomic<uint32> State = 0;
};

} // UE

#undef UE_API
