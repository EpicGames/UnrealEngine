// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Async/SharedLock.h"
#include <atomic>

#define UE_API CORE_API

namespace UE { class FSharedRecursiveMutex; }

namespace UE::Core::Private
{

struct FSharedRecursiveMutexLink
{
	[[nodiscard]] static bool Owns(const FSharedRecursiveMutex* Mutex);
	void Push(const FSharedRecursiveMutex* Mutex);
	void Pop();

	const FSharedRecursiveMutex* OwnedMutex = nullptr;
	FSharedRecursiveMutexLink* Next = nullptr;
};

} // UE::Core::Private

namespace UE
{

/**
 * An eight-byte shared mutex that is not fair and supports recursive locking.
 *
 * Prefer FRecursiveMutex when shared locking is not required.
 * Prefer FSharedMutex when recursive locking is not required.
 * All non-recursive shared locks will wait when any thread is waiting to take an exclusive lock.
 * An exclusive lock and a shared lock may not be simultaneously held by the same thread.
 */
class FSharedRecursiveMutex final
{
public:
	constexpr FSharedRecursiveMutex() = default;

	FSharedRecursiveMutex(const FSharedRecursiveMutex&) = delete;
	FSharedRecursiveMutex& operator=(const FSharedRecursiveMutex&) = delete;

	[[nodiscard]] inline bool IsLocked() const
	{
		return !!(State.load(std::memory_order_relaxed) & LockCountMask);
	}

	[[nodiscard]] UE_API bool IsLockedByCurrentThread() const;
	[[nodiscard]] UE_API bool TryLock();
	UE_API void Lock();
	UE_API void Unlock();

	[[nodiscard]] inline bool IsLockShared() const
	{
		return !!(State.load(std::memory_order_relaxed) & SharedLockCountMask);
	}

	// Use TSharedLock or TDynamicSharedLock to acquire a shared lock.
	[[nodiscard]] UE_API bool TryLockShared(Core::Private::FSharedRecursiveMutexLink& Link);
	UE_API void LockShared(Core::Private::FSharedRecursiveMutexLink& Link);
	UE_API void UnlockShared(Core::Private::FSharedRecursiveMutexLink& Link);

private:
	void LockSlow(uint32 CurrentState, uint32 CurrentThreadId);
	void LockSharedSlow(Core::Private::FSharedRecursiveMutexLink& Link);
	void WakeWaitingThreads(uint32 CurrentState);

	const void* GetLockAddress() const;
	const void* GetSharedLockAddress() const;

	static constexpr uint32 MayHaveWaitingLockFlag = 1 << 0;
	static constexpr uint32 MayHaveWaitingSharedLockFlag = 1 << 1;
	static constexpr uint32 LockCountShift = 2;
	static constexpr uint32 LockCountMask = 0x0000'0ffc;
	static constexpr uint32 SharedLockCountShift = 12;
	static constexpr uint32 SharedLockCountMask = 0xffff'f000;

	std::atomic<uint32> State = 0;
	std::atomic<uint32> ThreadId = 0;
};

template <>
class TSharedLock<FSharedRecursiveMutex> final
{
public:
	TSharedLock(const TSharedLock&) = delete;
	TSharedLock& operator=(const TSharedLock&) = delete;

	[[nodiscard]] inline explicit TSharedLock(FSharedRecursiveMutex& Lock)
		: Mutex(Lock)
	{
		Mutex.LockShared(Link);
	}

	inline ~TSharedLock()
	{
		Mutex.UnlockShared(Link);
	}

private:
	FSharedRecursiveMutex& Mutex;
	Core::Private::FSharedRecursiveMutexLink Link;
};

template <>
class TDynamicSharedLock<FSharedRecursiveMutex> final
{
public:
	TDynamicSharedLock() = default;

	TDynamicSharedLock(const TDynamicSharedLock&) = delete;
	TDynamicSharedLock& operator=(const TDynamicSharedLock&) = delete;

	[[nodiscard]] inline explicit TDynamicSharedLock(FSharedRecursiveMutex& Lock)
		: Mutex(&Lock)
	{
		Mutex->LockShared(Link);
		bLocked = true;
	}

	[[nodiscard]] inline explicit TDynamicSharedLock(FSharedRecursiveMutex& Lock, FDeferLock)
		: Mutex(&Lock)
	{
	}

	[[nodiscard]] inline TDynamicSharedLock(TDynamicSharedLock&& Other)
		: Mutex(Other.Mutex)
		, bLocked(Other.bLocked)
	{
		if (bLocked)
		{
			Mutex->LockShared(Link);
			Mutex->UnlockShared(Other.Link);
		}
		Other.Mutex = nullptr;
		Other.bLocked = false;
	}

	inline TDynamicSharedLock& operator=(TDynamicSharedLock&& Other)
	{
		if (bLocked)
		{
			Mutex->UnlockShared(Link);
		}
		Mutex = Other.Mutex;
		bLocked = Other.bLocked;
		if (bLocked)
		{
			Mutex->LockShared(Link);
			Mutex->UnlockShared(Other.Link);
		}
		Other.Mutex = nullptr;
		Other.bLocked = false;
		return *this;
	}

	inline ~TDynamicSharedLock()
	{
		if (bLocked)
		{
			Mutex->UnlockShared(Link);
		}
	}

	[[nodiscard]] bool TryLock()
	{
		check(!bLocked);
		check(Mutex);
		bLocked = Mutex->TryLockShared(Link);
		return bLocked;
	}

	void Lock()
	{
		check(!bLocked);
		check(Mutex);
		Mutex->LockShared(Link);
		bLocked = true;
	}

	void Unlock()
	{
		check(bLocked);
		bLocked = false;
		Mutex->UnlockShared(Link);
	}

	[[nodiscard]] inline bool OwnsLock() const
	{
		return bLocked;
	}

	inline explicit operator bool() const
	{
		return OwnsLock();
	}

private:
	FSharedRecursiveMutex* Mutex = nullptr;
	Core::Private::FSharedRecursiveMutexLink Link;
	bool bLocked = false;
};

} // UE

#undef UE_API
