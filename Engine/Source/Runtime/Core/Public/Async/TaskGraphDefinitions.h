// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

UE_DEPRECATED_HEADER(5.6, "Include Async/TaskGraphFwd.h");

// The old task graph interface was removed in UE 5.6
#define TASKGRAPH_NEW_FRONTEND 1
