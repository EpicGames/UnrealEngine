// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TaskGraphFwd.h: TaskGraph library
=============================================================================*/

#pragma once

#include "Templates/RefCounting.h"

class FBaseGraphTask;

/** Generic task used by the task graph system */
using FGraphEvent = FBaseGraphTask;

/** Convenience typedef for a reference counted pointer to a graph event */
using FGraphEventRef = TRefCountPtr<FBaseGraphTask>;
