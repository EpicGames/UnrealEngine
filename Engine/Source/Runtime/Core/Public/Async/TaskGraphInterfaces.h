// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TaskGraphInterfaces.h: TaskGraph library
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/Function.h"
#include "Delegates/Delegate.h"
#include "HAL/ThreadSafeCounter.h"
#include "Containers/LockFreeList.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Event.h"
#include "HAL/LowLevelMemTracker.h"
#include "Templates/RefCounting.h"
#include "Containers/LockFreeFixedSizeAllocator.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "Misc/MemStack.h"
#include "Misc/Timeout.h"
#include "Templates/Atomic.h"
#include "Templates/Models.h"
#include "ProfilingDebugging/MetadataTrace.h"

#include "Async/Fundamental/Task.h"

#include "Async/TaskGraphFwd.h"
#include "Async/TaskTrace.h"
#include "Tasks/TaskPrivate.h"
#include "Async/InheritedContext.h"

#if !defined(STATS)
#error "STATS must be defined as either zero or one."
#endif

// what level of checking to perform...normally checkSlow but could be ensure or check
#define checkThreadGraph checkSlow

//#define checkThreadGraph(x) ((x)||((*(char*)3) = 0))

DECLARE_STATS_GROUP(TEXT("Task Graph Tasks"), STATGROUP_TaskGraphTasks, STATCAT_Advanced);

DECLARE_CYCLE_STAT_EXTERN(TEXT("FReturnGraphTask"), STAT_FReturnGraphTask, STATGROUP_TaskGraphTasks, CORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("FTriggerEventGraphTask"), STAT_FTriggerEventGraphTask, STATGROUP_TaskGraphTasks, CORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("ParallelFor"), STAT_ParallelFor, STATGROUP_TaskGraphTasks, CORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("ParallelForTask"), STAT_ParallelForTask, STATGROUP_TaskGraphTasks, CORE_API);

namespace ENamedThreads
{
	enum Type : int32
	{
		UnusedAnchor = -1,
		/** The always-present, named threads are listed next **/
		RHIThread,
		GameThread,
		// The render thread is sometimes the game thread and is sometimes the actual rendering thread
		ActualRenderingThread = GameThread + 1,
		// CAUTION ThreadedRenderingThread must be the last named thread, insert new named threads before it

		/** not actually a thread index. Means "Unknown Thread" or "Any Unnamed Thread" **/
		AnyThread = 0xff, 

		/** High bits are used for a queue index and priority**/

		MainQueue =			0x000,
		LocalQueue =		0x100,

		NumQueues =			2,
		ThreadIndexMask =	0xff,
		QueueIndexMask =	0x100,
		QueueIndexShift =	8,

		/** High bits are used for a queue index task priority and thread priority**/

		NormalTaskPriority =	0x000,
		HighTaskPriority =		0x200,

		NumTaskPriorities =		2,
		TaskPriorityMask =		0x200,
		TaskPriorityShift =		9,

		NormalThreadPriority = 0x000,
		HighThreadPriority = 0x400,
		BackgroundThreadPriority = 0x800,

		NumThreadPriorities = 3,
		ThreadPriorityMask = 0xC00,
		ThreadPriorityShift = 10,

		/** Combinations **/
		GameThread_Local = GameThread | LocalQueue,
		ActualRenderingThread_Local = ActualRenderingThread | LocalQueue,

		AnyHiPriThreadNormalTask = AnyThread | HighThreadPriority | NormalTaskPriority,
		AnyHiPriThreadHiPriTask = AnyThread | HighThreadPriority | HighTaskPriority,

		AnyNormalThreadNormalTask = AnyThread | NormalThreadPriority | NormalTaskPriority,
		AnyNormalThreadHiPriTask = AnyThread | NormalThreadPriority | HighTaskPriority,

		AnyBackgroundThreadNormalTask = AnyThread | BackgroundThreadPriority | NormalTaskPriority,
		AnyBackgroundHiPriTask = AnyThread | BackgroundThreadPriority | HighTaskPriority,
	};

	struct FRenderThreadStatics
	{
	private:
		// These are private to prevent direct access by anything except the friend functions below
		static CORE_API TAtomic<Type> RenderThread;
		static CORE_API TAtomic<Type> RenderThread_Local;

		friend Type GetRenderThread();
		friend Type GetRenderThread_Local();
		friend void SetRenderThread(Type Thread);
		friend void SetRenderThread_Local(Type Thread);
	};

	UE_FORCEINLINE_HINT Type GetRenderThread()
	{
		return FRenderThreadStatics::RenderThread.Load(EMemoryOrder::Relaxed);
	}

	UE_FORCEINLINE_HINT Type GetRenderThread_Local()
	{
		return FRenderThreadStatics::RenderThread_Local.Load(EMemoryOrder::Relaxed);
	}

	UE_FORCEINLINE_HINT void SetRenderThread(Type Thread)
	{
		FRenderThreadStatics::RenderThread.Store(Thread, EMemoryOrder::Relaxed);
	}

	UE_FORCEINLINE_HINT void SetRenderThread_Local(Type Thread)
	{
		FRenderThreadStatics::RenderThread_Local.Store(Thread, EMemoryOrder::Relaxed);
	}

	// these allow external things to make custom decisions based on what sorts of task threads we are running now.
	// this are bools to allow runtime tuning.
	extern CORE_API int32 bHasBackgroundThreads; 
	extern CORE_API int32 bHasHighPriorityThreads;

	UE_FORCEINLINE_HINT Type GetThreadIndex(Type ThreadAndIndex)
	{
		return ((ThreadAndIndex & ThreadIndexMask) == AnyThread) ? AnyThread : Type(ThreadAndIndex & ThreadIndexMask);
	}

	UE_FORCEINLINE_HINT int32 GetQueueIndex(Type ThreadAndIndex)
	{
		return (ThreadAndIndex & QueueIndexMask) >> QueueIndexShift;
	}

	UE_FORCEINLINE_HINT int32 GetTaskPriority(Type ThreadAndIndex)
	{
		return (ThreadAndIndex & TaskPriorityMask) >> TaskPriorityShift;
	}

	inline int32 GetThreadPriorityIndex(Type ThreadAndIndex)
	{
		int32 Result = (ThreadAndIndex & ThreadPriorityMask) >> ThreadPriorityShift;
		check(Result >= 0 && Result < NumThreadPriorities);
		return Result;
	}

	inline Type SetPriorities(Type ThreadAndIndex, Type ThreadPriority, Type TaskPriority)
	{
		check(
			!(ThreadAndIndex & ~ThreadIndexMask) &&  // not a thread index
			!(ThreadPriority & ~ThreadPriorityMask) && // not a thread priority
			(ThreadPriority & ThreadPriorityMask) != ThreadPriorityMask && // not a valid thread priority
			!(TaskPriority & ~TaskPriorityMask) // not a task priority
			);
		return Type(ThreadAndIndex | ThreadPriority | TaskPriority);
	}

	inline Type SetPriorities(Type ThreadAndIndex, int32 PriorityIndex, bool bHiPri)
	{
		check(
			!(ThreadAndIndex & ~ThreadIndexMask) && // not a thread index
			PriorityIndex >= 0 && PriorityIndex < NumThreadPriorities // not a valid thread priority
			);
		return Type(ThreadAndIndex | (PriorityIndex << ThreadPriorityShift) | (bHiPri ? HighTaskPriority : NormalTaskPriority));
	}

	inline Type SetThreadPriority(Type ThreadAndIndex, Type ThreadPriority)
	{
		check(
			!(ThreadAndIndex & ~ThreadIndexMask) &&  // not a thread index
			!(ThreadPriority & ~ThreadPriorityMask) && // not a thread priority
			(ThreadPriority & ThreadPriorityMask) != ThreadPriorityMask // not a valid thread priority
			);
		return Type(ThreadAndIndex | ThreadPriority);
	}

	inline Type SetTaskPriority(Type ThreadAndIndex, Type TaskPriority)
	{
		check(
			!(ThreadAndIndex & ~ThreadIndexMask) &&  // not a thread index
			!(TaskPriority & ~TaskPriorityMask) // not a task priority
			);
		return Type(ThreadAndIndex | TaskPriority);
	}
}

DECLARE_INTRINSIC_TYPE_LAYOUT(ENamedThreads::Type);

class FAutoConsoleTaskPriority
{
	FString RawSetting;
	FString FullHelpText;
	FAutoConsoleVariableRef Variable;
	ENamedThreads::Type ThreadPriority;
	ENamedThreads::Type TaskPriority;
	ENamedThreads::Type TaskPriorityIfForcedToNormalThreadPriority;

	static FString CreateFullHelpText(const TCHAR* Name, const TCHAR* OriginalHelp);
	static FString ConfigStringFromPriorities(ENamedThreads::Type InThreadPriority, ENamedThreads::Type InTaskPriority, ENamedThreads::Type InTaskPriorityBackup);
	void OnSettingChanged(IConsoleVariable* Variable);

public:
	CORE_API FAutoConsoleTaskPriority(const TCHAR* Name, const TCHAR* Help, ENamedThreads::Type DefaultThreadPriority, ENamedThreads::Type DefaultTaskPriority, ENamedThreads::Type DefaultTaskPriorityIfForcedToNormalThreadPriority = ENamedThreads::UnusedAnchor);

	inline ENamedThreads::Type Get(ENamedThreads::Type Thread = ENamedThreads::AnyThread)
	{
		// if we don't have the high priority thread that was asked for, or we are downgrading thread priority due to power saving
		// then use a normal thread priority with the backup task priority
		if (ThreadPriority == ENamedThreads::HighThreadPriority && !ENamedThreads::bHasHighPriorityThreads)
		{
			return ENamedThreads::SetTaskPriority(Thread, TaskPriorityIfForcedToNormalThreadPriority);
		}
		// if we don't have the background priority thread that was asked for, then use a normal thread priority with the backup task priority
		if (ThreadPriority == ENamedThreads::BackgroundThreadPriority && !ENamedThreads::bHasBackgroundThreads)
		{
			return ENamedThreads::SetTaskPriority(Thread, TaskPriorityIfForcedToNormalThreadPriority);
		}
		return ENamedThreads::SetPriorities(Thread, ThreadPriority, TaskPriority);
	}
};


namespace ESubsequentsMode
{
	enum Type
	{
		/** Necessary when another task will depend on this task. */
		TrackSubsequents,
		/** Can be used to save task graph overhead when firing off a task that will not be a dependency of other tasks. */
		FireAndForget
	};
}

/** Convenience typedef for a an array a graph events **/
typedef TArray<FGraphEventRef, TInlineAllocator<4> > FGraphEventArray;

/** returns trace IDs of given tasks **/
CORE_API TArray<TaskTrace::FId> GetTraceIds(const FGraphEventArray& Tasks);

/** Interface to the task graph system **/
class FTaskGraphInterface
{
	friend class FBaseGraphTask;
	friend UE::Tasks::Private::FTaskBase;

	/**
	 *	Internal function to queue a task
	 *	@param	Task; the task to queue
	 *	@param	ThreadToExecuteOn; Either a named thread for a threadlocked task or ENamedThreads::AnyThread for a task that is to run on a worker thread
	 *	@param	CurrentThreadIfKnown; This should be the current thread if it is known, or otherwise use ENamedThreads::AnyThread and the current thread will be determined.
	**/
	virtual void QueueTask(class FBaseGraphTask* Task, bool bWakeUpWorker, ENamedThreads::Type ThreadToExecuteOn, ENamedThreads::Type CurrentThreadIfKnown = ENamedThreads::AnyThread) = 0;

public:

	virtual ~FTaskGraphInterface() = default;

	// Startup, shutdown and singleton API

	/** 
	 *	Explicit start call for the system. The ordinary singleton pattern does not work because internal threads start asking for the singleton before the constructor has returned.
	 *	@param	NumThreads; Total number of threads in the system, must be 0 to disable separate taskgraph thread, at least 2 if !threadedrendering, else at least 3
	**/
	static CORE_API void Startup(int32 NumThreads);
	/** 
	 *	Explicit start call to shutdown the system. This is unlikely to work unless the system is idle.
	**/
	static CORE_API void Shutdown();
    /**
     *	Check to see if the system is running.
     **/
    static CORE_API bool IsRunning();
	/**
	 *	Singleton for the system
	 *	@return a reference to the task graph system
	**/
	static CORE_API FTaskGraphInterface& Get();

	/**
	* The task graph is always multi-threaded for platforms that support it.
	* For forked processes, the taskgraph will be singlethread for the master process but becomes multithread in the forked process
	*/
	static bool IsMultithread();

	/** Return the current thread type, if known. **/
	virtual ENamedThreads::Type GetCurrentThreadIfKnown(bool bLocalQueue = false) = 0;

	/** Return true if the current thread is known. **/
	virtual bool IsCurrentThreadKnown() = 0;

	/** 
		Return the number of worker (non-named) threads PER PRIORITY SET.
		This is useful for determining how many tasks to split a job into.
	**/
	virtual	int32 GetNumWorkerThreads() = 0;

	/**
		Return the number of foreground worker threads.
		If the old backend is used, return the number of high-pri workers (0 if high-pri workers are disabled).
	**/
	virtual	int32 GetNumForegroundThreads() = 0;

	/**
		Return the number of background worker threads.
		If the old backend is used, return the number of background workers (0 if background workers are disabled).
	**/
	virtual	int32 GetNumBackgroundThreads() = 0;

	/** Return true if the given named thread is processing tasks. This is only a "guess" if you ask for a thread other than yourself because that can change before the function returns. **/
	virtual bool IsThreadProcessingTasks(ENamedThreads::Type ThreadToCheck) = 0;

	// External Thread API

	/** 
	 *	A one time call that "introduces" an external thread to the system. Basically, it just sets up the TLS info 
	 *	@param	CurrentThread; The name of the current thread.
	**/
	virtual void AttachToThread(ENamedThreads::Type CurrentThread)=0;

	/** 
	 *	Requests that a named thread, which must be this thread, run until idle, then return.
	 *	@param	CurrentThread; The name of this thread
	**/
	virtual uint64 ProcessThreadUntilIdle(ENamedThreads::Type CurrentThread)=0;

	/** 
	 *	Requests that a named thread, which must be this thread, run until an explicit return request is received, then return.
	 *	@param	CurrentThread; The name of this thread
	**/
	virtual void ProcessThreadUntilRequestReturn(ENamedThreads::Type CurrentThread)=0;

	/** 
	 *	Request that the given thread stop when it is idle
	**/
	virtual void RequestReturn(ENamedThreads::Type CurrentThread)=0;

	/** 
	 *	Requests that a named thread, which must be this thread, run until a list of tasks is complete.
	 *	This function assumes that most of the tasks will be processed on a different thread, but will process if needed.
	 *	@param	Tasks - tasks to wait for
	 *	@param	CurrentThread - This thread, must be a named thread
	**/
	virtual void WaitUntilTasksComplete(const FGraphEventArray& Tasks, ENamedThreads::Type CurrentThreadIfKnown = ENamedThreads::AnyThread)=0;
	

	/** Used to define what ProcessUntilTasksComplete should do next */
	enum class EProcessTasksOperation : int8
	{
		/** Default behavior with no update callback, try to process all other thread thread tasks in the queue before checking named thread tasks */
		ProcessAllOtherTasks,

		/** Try to process other thread tasks, but immediately process named thread tasks after completing one */
		ProcessOneOtherTask,

		/** Immediately try to process named thread tasks and call update again */
		ProcessNamedThreadTasks,

		/** Stop any idle processing and wait on this thread until all tasks are complete, update is not called again */
		WaitUntilComplete,

		/** Stop processing entirely and return, like if ProcessingTimeout has expired */
		StopProcessing,
	};

	/** Function periodically called during task processing, with parameter set to number of tasks remaining */
	typedef TFunction<EProcessTasksOperation(int32 /*NumTasksRemaining*/)> FProcessTasksUpdateCallback;

	/**
	 *	Requests that a named thread actively attempt to process a list of tasks and periodically call an update function when idle.
	 *	This function assumes that many of the tasks can be completed on this thread.
	 *	@param	Tasks - tasks to wait for
	 *	@param	CurrentThread - This thread, must be a named thread
	 *	@param	IdleWorkUpdate - If set, call this function after trying to process named thread work if any tasks are stalled
	 *	@return true if all tasks were completed, return false if StopProcessing was returned or it failed to process all tasks
	**/
	virtual bool ProcessUntilTasksComplete(const FGraphEventArray& Tasks, ENamedThreads::Type CurrentThreadIfKnown, const FProcessTasksUpdateCallback& IdleWorkUpdate = {}) = 0;

	/** 
	 *	When a set of tasks complete, fire a scoped event
	 *	@param	InEvent - event to fire when the task is done
	 *	@param	Tasks - tasks to wait for
	 *	@param	CurrentThreadIfKnown - This thread, if known
	**/
	virtual void TriggerEventWhenTasksComplete(FEvent* InEvent, const FGraphEventArray& Tasks, ENamedThreads::Type CurrentThreadIfKnown = ENamedThreads::AnyThread, ENamedThreads::Type TriggerThread = ENamedThreads::AnyHiPriThreadHiPriTask)=0;

	/** 
	 *	Requests that a named thread, which must be this thread, run until a task is complete
	 *	@param	Task - task to wait for
	 *	@param	CurrentThread - This thread, must be a named thread
	**/
	void WaitUntilTaskCompletes(const FGraphEventRef& Task, ENamedThreads::Type CurrentThreadIfKnown = ENamedThreads::AnyThread)
	{
		WaitUntilTasksComplete({ Task }, CurrentThreadIfKnown);
	}

	void WaitUntilTaskCompletes(FGraphEventRef&& Task, ENamedThreads::Type CurrentThreadIfKnown = ENamedThreads::AnyThread)
	{
		WaitUntilTasksComplete({ MoveTemp(Task) }, CurrentThreadIfKnown);
	}

	/** 
	 *	When a task completes, fire a scoped event
	 *	@param	InEvent - event to fire when the task is done
	 *	@param	Task - task to wait for
	 *	@param	CurrentThreadIfKnown - This thread, if known
	**/
	void TriggerEventWhenTaskCompletes(FEvent* InEvent, const FGraphEventRef& Task, ENamedThreads::Type CurrentThreadIfKnown = ENamedThreads::AnyThread, ENamedThreads::Type TriggerThread = ENamedThreads::AnyHiPriThreadHiPriTask)
	{
		FGraphEventArray Prerequistes;
		Prerequistes.Add(Task);
		TriggerEventWhenTasksComplete(InEvent, Prerequistes, CurrentThreadIfKnown, TriggerThread);
	}

	virtual FBaseGraphTask* FindWork(ENamedThreads::Type ThreadInNeed) = 0;

	virtual void StallForTuning(int32 Index, bool Stall) = 0;

	/**
	*	Delegates for shutdown
	*	@param	Callback - function to call prior to shutting down the taskgraph
	**/
	virtual void AddShutdownCallback(TFunction<void()>& Callback) = 0;

	virtual void WakeNamedThread(ENamedThreads::Type ThreadToWake) = 0;

	/**
	*	A (slow) function to call a function on every known thread, both named and workers
	*	@param	Callback - function to call prior to shutting down the taskgraph
	**/
	static void BroadcastSlow_OnlyUseForSpecialPurposes(bool bDoTaskThreads, bool bDoBackgroundThreads, TFunction<void(ENamedThreads::Type CurrentThread)>& Callback);
};

struct FTaskGraphBlockAllocationTag : FDefaultBlockAllocationTag
{
	static constexpr uint32 BlockSize = 64 * 1024;
	static constexpr bool AllowOversizedBlocks = false;
	static constexpr bool RequiresAccurateSize = false;
	static constexpr bool InlineBlockAllocation = true;
	static constexpr const char* TagName = "TaskGraphLinear";

	using Allocator = TBlockAllocationCache<BlockSize, FAlignedAllocator>;
};

/** Base class for all graph tasks, used for both TGraphTask and simple graph events. This is a wrapper around the `Tasks::Private::FTaskBase` functionality */
class FBaseGraphTask : public UE::Tasks::Private::FTaskBase
{
public:
	// These functions handle integration with FTaskBase and the named thread executor and should not be called directly
	explicit FBaseGraphTask(const FGraphEventArray* InPrerequisites)
		: FTaskBase(/*InitRefCount=*/ 1, false /* bUnlockPrerequisites */)
	{
		if (InPrerequisites != nullptr)
		{
			AddPrerequisites(*InPrerequisites, false /* bLockPrerequisite */);
		}

		UnlockPrerequisites();
	}

	void Init(const TCHAR* InDebugName, UE::Tasks::ETaskPriority InPriority, UE::Tasks::EExtendedTaskPriority InExtendedPriority, UE::Tasks::ETaskFlags InTaskFlags = UE::Tasks::ETaskFlags::None)
	{
		FTaskBase::Init(InDebugName, InPriority, InExtendedPriority, InTaskFlags);
	}

	inline void Execute(TArray<FBaseGraphTask*>& NewTasks, ENamedThreads::Type CurrentThread, bool bDeleteOnCompletion)
	{	// only called for named thread tasks, normal tasks are executed using `FTaskBase` API directly (see `TGraphTask`)
		checkSlow(NewTasks.Num() == 0);
		checkSlow(bDeleteOnCompletion);
		checkSlow(IsNamedThreadTask());
		TryExecuteTask(); // It's possible retraction already executed this task
		ReleaseInternalReference(); // named tasks are executed by named threads, outside of the scheduler
	}


	/** Returns a reference to this task that can be used as a prerequisite for new tasks or passed to DontCompleteUntil */
	FGraphEventRef GetCompletionEvent()
	{
		return this;
	}

	/** 
	 * Call on a currently active task to add a nested task, which will delay any subsequent tasks until the nested task completes.
	 * AddPrerequisites should be used instead if the task has not yet been launched.
	 */
	void DontCompleteUntil(FGraphEventRef NestedTask)
	{
		if (!NestedTask)
		{
			return;
		}

		if (IsTaskEvent())
		{	// TaskEvent can't have nested tasks, add it as a prerequisite instead to retain backward compatibility
			AddPrerequisites(*NestedTask);
		}
		else
		{
			checkSlow(UE::Tasks::Private::GetCurrentTask() == this); // a nested task can be added only from inside of parent's execution
			AddNested(*NestedTask);
		}
	}

	/** Return true if this task has finished executing, this wrapper exists for backward compatibility */
	bool IsComplete() const
	{
		return IsCompleted();
	}

	/** Create a simple task event that can be used as a prerequisite for other tasks, and then manually triggered with DispatchSubsequents */
	static FGraphEventRef CreateGraphEvent();

	/** Returns true if this is a simple task event, which cannot execute code and must be manually dispatched */
	bool IsTaskEvent() const
	{
		return GetExtendedPriority() == UE::Tasks::EExtendedTaskPriority::TaskEvent;
	}

	/** 
	 * Unlocks a task that was returned from ConstructAndHold or CreateGraphEvent, which may execute immediately if prerequisites are already completed.
	 * This can only be safely called exactly once and the caller is responsible for tracking that.
	 */
	void Unlock(ENamedThreads::Type CurrentThreadIfKnown = ENamedThreads::AnyThread)
	{
		if (IsTaskEvent())
		{
			// An event is not "in the system" until it's triggered, and should be kept alive only by external references. Once it's triggered it's in the system 
			// and can outlive external references, so we need to keep it alive by holding an internal reference. It will be released when the event is signaled
			AddRef();
		}
		TryLaunch(0);
	}

	/** Backward compatibility wrapper for Unlock */
	void DispatchSubsequents(ENamedThreads::Type CurrentThreadIfKnown = ENamedThreads::AnyThread)
	{
		Unlock(CurrentThreadIfKnown);
	}

	void SetDebugName(const TCHAR* InDebugName)
	{	// incompatible with the new API that requires debug name during task construction and doesn't allow to set it later. "debug name" feature was added
		// to the old API recently, is used only in a couple of places and will be fixed manually by switching to the new API
	}

	/** Blocks the current thread and waits for the completion of this task */
	void Wait(ENamedThreads::Type CurrentThreadIfKnown = ENamedThreads::AnyThread)
	{
		// local queue have to be handled by the original TaskGraph implementation. Tasks System doesn't support local queues
		if (ENamedThreads::GetQueueIndex(CurrentThreadIfKnown) != ENamedThreads::MainQueue)
		{
			return FTaskGraphInterface::Get().WaitUntilTaskCompletes(this, CurrentThreadIfKnown);
		}

		FTaskBase::WaitWithNamedThreadsSupport();
	}

	/** Returns the old-style named thread this task will be executed on */
	ENamedThreads::Type GetThreadToExecuteOn() const
	{
		return UE::Tasks::Private::TranslatePriority(GetPriority(), GetExtendedPriority());
	}
};

/** Templated graph task that is created to execute a specific function */
template<typename TTask>
class TGraphTask final : public TConcurrentLinearObject<TGraphTask<TTask>, FTaskGraphBlockAllocationTag>, public FBaseGraphTask
{
public:
	/**
	 *	This is a helper class returned from the factory. It constructs the embeded task with a set of arguments and sets the task up and makes it ready to execute.
	 *	The task may complete before these routines even return.
	 **/
	class FConstructor
	{
	public:
		UE_NONCOPYABLE(FConstructor);

		/**
		 * Constructs a task and immediately dispatches the task for possible execution if prerequisites have completed.
		 * Note! Generally speaking references will not pass through; use pointers
		 */
		template<typename...T>
		inline FGraphEventRef ConstructAndDispatchWhenReady(T&&... Args)
		{
			FGraphEventRef Ref{ ConstructAndHoldImpl(Forward<T>(Args)...) };
			Ref->TryLaunch(sizeof(TGraphTask));
			return Ref;
		}

		/**
		 * Constructs a task and holds it for later dispatch by calling Unlock.
		 * Note! Generally speaking references will not pass through; use pointers
		 */
		template<typename...T>
		inline TGraphTask* ConstructAndHold(T&&... Args)
		{
			TGraphTask* Task = ConstructAndHoldImpl(Forward<T>(Args)...);
			TaskTrace::Created(Task->GetTraceId(), sizeof(*Task));
			return Task;
		}

		FConstructor(const FGraphEventArray* InPrerequisites)
			: Prerequisites(InPrerequisites)
		{
		}

	private:
		struct CGetTaskFlagsProvider
		{
			template <typename TaskType>
			auto Requires(TaskType& Task) -> decltype(
				Task.GetTaskFlags()
			);
		};

		template<typename...T>
		inline TGraphTask* ConstructAndHoldImpl(T&&... Args)
		{
			TGraphTask* Task = new TGraphTask(Prerequisites);
			TTask* TaskObject = new(&Task->TaskStorage) TTask(Forward<T>(Args)...);

			UE::Tasks::ETaskPriority Pri;
			UE::Tasks::EExtendedTaskPriority ExtPri;
			UE::Tasks::Private::TranslatePriority(TaskObject->GetDesiredThread(), Pri, ExtPri);

			// Only use this when TTask implements GetTaskFlags().
			if constexpr (TModels_V<CGetTaskFlagsProvider, TTask>)
			{
				Task->Init(Pri, ExtPri, TaskObject->GetTaskFlags());
			}
			else
			{
				Task->Init(Pri, ExtPri);
			}

			return Task;
		}

	private:
		const FGraphEventArray* Prerequisites;
	};

	/**
	 *	Factory to create a task and return the helper object to construct the embedded task and set it up for execution.
	 *	@param Prerequisites; the list of FGraphEvents that must be completed prior to this task executing.
	 *	@param CurrentThreadIfKnown; provides the index of the thread we are running on. Can be ENamedThreads::AnyThread if the current thread is unknown.
	 *	@return a temporary helper class which can be used to complete the process.
	**/
	static FConstructor CreateTask(const FGraphEventArray* Prerequisites = nullptr, ENamedThreads::Type CurrentThreadIfKnown = ENamedThreads::AnyThread)
	{
		return FConstructor(Prerequisites);
	}

private:
	explicit TGraphTask(const FGraphEventArray* InPrerequisites)
		: FBaseGraphTask(InPrerequisites)
	{
	}

	void Init(UE::Tasks::ETaskPriority InPriority, UE::Tasks::EExtendedTaskPriority InExtendedPriority, UE::Tasks::ETaskFlags InTaskFlags = UE::Tasks::ETaskFlags::None)
	{
		FBaseGraphTask::Init(TEXT("GraphTask"), InPriority, InExtendedPriority, InTaskFlags);
	}

	struct CGetStatIdProvider
	{
		template <typename TaskType>
		auto Requires(TaskType& Task) -> decltype(
			Task.GetStatId()
		);
	};

	virtual void ExecuteTask() override final
	{
		FGraphEventRef GraphEventRef{ this };
		TTask* TaskObject = TaskStorage.GetTypedPtr();
		// Only use this scope when GetStatId() is actually implemented on TaskObject for backward compat, sometimes it's not.
		if constexpr (TModels_V<CGetStatIdProvider, TTask>)
		{
			SCOPE_CYCLE_COUNTER_STATID(TaskObject->GetStatId());
			TaskObject->DoTask(TaskObject->GetDesiredThread(), GraphEventRef);
		}
		else
		{
			TaskObject->DoTask(TaskObject->GetDesiredThread(), GraphEventRef);
		}
		DestructItem(TaskObject);
	}

private:
	TTypeCompatibleBytes<TTask> TaskStorage;
};

// an adaptation of FBaseGraphTask to be used as a standalone FGraphEvent
class FGraphEventImpl : public FBaseGraphTask
{
public:
	FGraphEventImpl()
		: FBaseGraphTask(nullptr)
	{
		TaskTrace::Created(GetTraceId(), sizeof(*this));
		Init(TEXT("GraphEvent"), UE::Tasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::TaskEvent);
	}

	static void* operator new(size_t Size);
	static void operator delete(void* Ptr);

private:
	virtual void ExecuteTask() override final
	{
		checkNoEntry(); // graph events are never executed
	}
};

using FGraphEventImplAllocator = TLockFreeFixedSizeAllocator_TLSCache<sizeof(FGraphEventImpl), PLATFORM_CACHE_LINE_SIZE>;
CORE_API FGraphEventImplAllocator& GetGraphEventImplAllocator();

inline void* FGraphEventImpl::operator new(size_t Size)
{
	return GetGraphEventImplAllocator().Allocate();
}

inline void FGraphEventImpl::operator delete(void* Ptr)
{
	GetGraphEventImplAllocator().Free(Ptr);
}

inline FGraphEventRef FBaseGraphTask::CreateGraphEvent()
{
	FGraphEventImpl* GraphEvent = new FGraphEventImpl;
	return FGraphEventRef{ GraphEvent, /*bAddRef = */ false };
}

// Blocks the current thread until any of the given tasks is completed.
// Is slightly more efficient than `AnyTaskCompleted()->Wait()` and supports timeout while `FGraphEvent::Wait()` doesn't.
// Returns the index of the first completed task, or `INDEX_NONE` on timeout.
CORE_API int32 WaitForAnyTaskCompleted(const FGraphEventArray& GraphEvents, FTimespan Timeout = FTimespan::MaxValue());

// Returns a graph event that gets completed as soon as any of the given tasks gets completed
CORE_API FGraphEventRef AnyTaskCompleted(const FGraphEventArray& GraphEvents);

/** 
 *	FReturnGraphTask is a task used to return flow control from a named thread back to the original caller of ProcessThreadUntilRequestReturn
 **/
class FReturnGraphTask
{
public:

	/** 
	 *	Constructor
	 *	@param InThreadToReturnFrom; named thread to cause to return
	**/
	FReturnGraphTask(ENamedThreads::Type InThreadToReturnFrom)
		: ThreadToReturnFrom(InThreadToReturnFrom)
	{
		checkThreadGraph(ThreadToReturnFrom != ENamedThreads::AnyThread); // doesn't make any sense to return from any thread
	}

	UE_FORCEINLINE_HINT TStatId GetStatId() const
	{
		return GET_STATID(STAT_FReturnGraphTask);
	}

	/** 
	 *	Retrieve the thread that this task wants to run on.
	 *	@return the thread that this task should run on.
	 **/
	ENamedThreads::Type GetDesiredThread()
	{
		return ThreadToReturnFrom;
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	/** 
	 *	Actually execute the task.
	 *	@param	CurrentThread; the thread we are running on
	 *	@param	MyCompletionGraphEvent; my completion event. Not always useful since at the end of DoWork, you can assume you are done and hence further tasks do not need you as a prerequisite. 
	 *	However, MyCompletionGraphEvent can be useful for passing to other routines or when it is handy to set up subsequents before you actually do work.
	 **/
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		checkThreadGraph(ENamedThreads::GetThreadIndex(ThreadToReturnFrom) == ENamedThreads::GetThreadIndex(CurrentThread)); // we somehow are executing on the wrong thread.
		FTaskGraphInterface::Get().RequestReturn(ThreadToReturnFrom);
	}

private:
	/** Named thread that we want to cause to return to the caller of ProcessThreadUntilRequestReturn. **/
	ENamedThreads::Type ThreadToReturnFrom;
};

/**
 * Class that generalizes functionality of storing and exposing custom stat id.
 * Should only be used as a base of a task graph class with custom stat id.
 */
class FCustomStatIDGraphTaskBase
{
public:
	/** 
	 * Constructor.
	 *
	 * @param StatId The stat id for this task.
	 */
	FCustomStatIDGraphTaskBase(const TStatId& StatId)
	{
#if STATS || ENABLE_STATNAMEDEVENTS
		StatID = StatId;
#endif
	}

	/** 
	 * Gets stat id for this task.
	 *
	 * @return Stat id.
	 */
	inline TStatId GetStatId() const
	{
#if STATS|| ENABLE_STATNAMEDEVENTS
		return StatID;
#else
		return TStatId();
#endif
	}

private:
#if STATS|| ENABLE_STATNAMEDEVENTS
	/** Stat id of this object. */
	TStatId StatID;
#endif

};

/** 
 *	FNullGraphTask is a task that does nothing. It can be used to "gather" tasks into one prerequisite.
 **/
class FNullGraphTask : public FCustomStatIDGraphTaskBase
{
public:
	/** 
	 *	Constructor
	 *	@param StatId The stat id for this task.
	 *	@param InDesiredThread; Thread to run on, can be ENamedThreads::AnyThread
	**/
	FNullGraphTask(const TStatId& StatId, ENamedThreads::Type InDesiredThread)
		: FCustomStatIDGraphTaskBase(StatId)
		, DesiredThread(InDesiredThread)
	{
	}

	/** 
	 *	Retrieve the thread that this task wants to run on.
	 *	@return the thread that this task should run on.
	 **/
	ENamedThreads::Type GetDesiredThread()
	{
		return DesiredThread;
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	/** 
	 *	Actually execute the task.
	 *	@param	CurrentThread; the thread we are running on
	 *	@param	MyCompletionGraphEvent; my completion event. Not always useful since at the end of DoWork, you can assume you are done and hence further tasks do not need you as a prerequisite. 
	 *	However, MyCompletionGraphEvent can be useful for passing to other routines or when it is handy to set up subsequents before you actually do work.
	 **/
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
	}
private:
	/** Thread to run on, can be ENamedThreads::AnyThread **/
	ENamedThreads::Type DesiredThread;
};

/** 
 * FTriggerEventGraphTask is a task that triggers an event
 */
class FTriggerEventGraphTask
{
public:
	/** 
	 *	Constructor
	 *	@param InScopedEvent; Scoped event to fire
	**/
	FTriggerEventGraphTask(FEvent* InEvent, ENamedThreads::Type InDesiredThread = ENamedThreads::AnyHiPriThreadHiPriTask)
		: Event(InEvent) 
		, DesiredThread(InDesiredThread)
	{
		check(Event);
	}

	UE_FORCEINLINE_HINT TStatId GetStatId() const
	{
		return GET_STATID(STAT_FTriggerEventGraphTask);
	}

	ENamedThreads::Type GetDesiredThread()
	{
		return DesiredThread;
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		Event->Trigger();
	}
private:
	FEvent* Event;
	/** Thread to run on, can be ENamedThreads::AnyThread **/
	ENamedThreads::Type DesiredThread;
};

/** Task class for simple delegate based tasks. This is less efficient than a custom task, doesn't provide the task arguments, doesn't allow specification of the current thread, etc. **/
class FSimpleDelegateGraphTask : public FCustomStatIDGraphTaskBase
{
public:
	DECLARE_DELEGATE(FDelegate);

	/** Delegate to fire when task runs **/
	FDelegate							TaskDelegate;
	/** Thread to run delegate on **/
	const ENamedThreads::Type			DesiredThread;
public:
	ENamedThreads::Type GetDesiredThread()
	{
		return DesiredThread;
	}
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		TaskDelegate.ExecuteIfBound();
	}
	/**
	  * Task constructor
	  * @param InTaskDelegate - delegate to execute when the prerequisites are complete
	  *	@param StatId The stat id for this task.
	  * @param InDesiredThread - Thread to run on
	**/
	FSimpleDelegateGraphTask(const FDelegate& InTaskDeletegate, const TStatId StatId, ENamedThreads::Type InDesiredThread)
		: FCustomStatIDGraphTaskBase(StatId)
		, TaskDelegate(InTaskDeletegate)
		, DesiredThread(InDesiredThread)
	{
	}

	/**
	  * Create a task and dispatch it when the prerequisites are complete
	  * @param InTaskDelegate - delegate to execute when the prerequisites are complete
	  * @param InStatId - StatId of task for debugging or analysis tools
	  * @param InPrerequisites - Handles for prerequisites for this task, can be NULL if there are no prerequisites
	  * @param InDesiredThread - Thread to run on
	  * @return completion handle for the new task 
	**/
	static FGraphEventRef CreateAndDispatchWhenReady(const FDelegate& InTaskDeletegate, const TStatId InStatId, const FGraphEventArray* InPrerequisites = NULL, ENamedThreads::Type InDesiredThread = ENamedThreads::AnyThread)
	{
		return TGraphTask<FSimpleDelegateGraphTask>::CreateTask(InPrerequisites).ConstructAndDispatchWhenReady<const FDelegate&>(InTaskDeletegate, InStatId, InDesiredThread);
	}
	/**
	  * Create a task and dispatch it when the prerequisites are complete
	  * @param InTaskDelegate - delegate to execute when the prerequisites are complete
	  * @param InStatId - StatId of task for debugging or analysis tools
	  * @param InPrerequisite - Handle for a single prerequisite for this task
	  * @param InDesiredThread - Thread to run on
	  * @return completion handle for the new task 
	**/
	static FGraphEventRef CreateAndDispatchWhenReady(const FDelegate& InTaskDeletegate, const TStatId&& InStatId, const FGraphEventRef& InPrerequisite, ENamedThreads::Type InDesiredThread = ENamedThreads::AnyThread)
	{
		FGraphEventArray Prerequisites;
		check(InPrerequisite.GetReference());
		Prerequisites.Add(InPrerequisite);
		return CreateAndDispatchWhenReady(InTaskDeletegate, InStatId, &Prerequisites, InDesiredThread);
	}
};

/** Task class for more full featured delegate based tasks. Still less efficient than a custom task, but provides all of the args **/
class FDelegateGraphTask : public FCustomStatIDGraphTaskBase
{
public:
	DECLARE_DELEGATE_TwoParams( FDelegate,ENamedThreads::Type, const  FGraphEventRef& );

	/** Delegate to fire when task runs **/
	FDelegate							TaskDelegate;
	/** Thread to run delegate on **/
	const ENamedThreads::Type			DesiredThread;
public:
	ENamedThreads::Type GetDesiredThread()
	{
		return DesiredThread;
	}
	static ESubsequentsMode::Type GetSubsequentsMode() 
	{ 
		return ESubsequentsMode::TrackSubsequents; 
	}
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		TaskDelegate.ExecuteIfBound(CurrentThread, MyCompletionGraphEvent);
	}
	/**
	  * Task constructor
	  * @param InTaskDelegate - delegate to execute when the prerequisites are complete
	  *	@param InStatId - The stat id for this task.
	  * @param InDesiredThread - Thread to run on
	**/
	FDelegateGraphTask(const FDelegate& InTaskDeletegate, const TStatId InStatId, ENamedThreads::Type InDesiredThread)
		: FCustomStatIDGraphTaskBase(InStatId)
		, TaskDelegate(InTaskDeletegate)
		, DesiredThread(InDesiredThread)
	{
	}

	/**
	  * Create a task and dispatch it when the prerequisites are complete
	  * @param InTaskDelegate - delegate to execute when the prerequisites are complete
	  *	@param InStatId - The stat id for this task.
	  * @param InPrerequisites - Handles for prerequisites for this task, can be NULL if there are no prerequisites
	  * @param InCurrentThreadIfKnown - This thread, if known
	  * @param InDesiredThread - Thread to run on
	  * @return completion handle for the new task 
	**/
	static FGraphEventRef CreateAndDispatchWhenReady(const FDelegate& InTaskDeletegate, const TStatId InStatId, const FGraphEventArray* InPrerequisites = NULL, ENamedThreads::Type InCurrentThreadIfKnown = ENamedThreads::AnyThread, ENamedThreads::Type InDesiredThread = ENamedThreads::AnyThread)
	{
		return TGraphTask<FDelegateGraphTask>::CreateTask(InPrerequisites, InCurrentThreadIfKnown).ConstructAndDispatchWhenReady<const FDelegate&>(InTaskDeletegate, InStatId, InDesiredThread);
	}
	/**
	  * Create a task and dispatch it when the prerequisites are complete
	  * @param InTaskDelegate - delegate to execute when the prerequisites are complete
	  *	@param InStatId - The stat id for this task.
	  * @param InPrerequisite - Handle for a single prerequisite for this task
	  * @param InCurrentThreadIfKnown - This thread, if known
	  * @param InDesiredThread - Thread to run on
	  * @return completion handle for the new task 
	**/
	static FGraphEventRef CreateAndDispatchWhenReady(const FDelegate& InTaskDeletegate, const TStatId InStatId, const FGraphEventRef& InPrerequisite, ENamedThreads::Type InCurrentThreadIfKnown = ENamedThreads::AnyThread, ENamedThreads::Type InDesiredThread = ENamedThreads::AnyThread)
	{
		FGraphEventArray Prerequisites;
		check(InPrerequisite.GetReference());
		Prerequisites.Add(InPrerequisite);
		return CreateAndDispatchWhenReady(InTaskDeletegate, InStatId, &Prerequisites, InCurrentThreadIfKnown, InDesiredThread);
	}
};

/** Task class for lambda based tasks. **/
template<typename Signature, ESubsequentsMode::Type SubsequentsMode>
class TFunctionGraphTaskImpl : public FCustomStatIDGraphTaskBase
{
private:
	/** Function to run **/
	TUniqueFunction<Signature> Function;
	/** Thread to run the function on **/
	const ENamedThreads::Type DesiredThread;

public:
	/**
	 * Task constructor
	 * @param InFunction - function to execute when the prerequisites are complete
	 *	@param StatId The stat id for this task.
	 * @param InDesiredThread - Thread to run on
	 **/
	TFunctionGraphTaskImpl(TUniqueFunction<Signature>&& InFunction, TStatId StatId, ENamedThreads::Type InDesiredThread)
		: FCustomStatIDGraphTaskBase(StatId),
		Function(MoveTemp(InFunction)),
		DesiredThread(InDesiredThread)
	{}

	ENamedThreads::Type GetDesiredThread() const
	{
		return DesiredThread;
	}

	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return SubsequentsMode;
	}

	UE_FORCEINLINE_HINT void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		DoTaskImpl(Function, CurrentThread, MyCompletionGraphEvent);
	}

private:
	UE_FORCEINLINE_HINT static void DoTaskImpl(TUniqueFunction<void()>& Function, ENamedThreads::Type CurrentThread,
		const FGraphEventRef& MyCompletionGraphEvent)
	{
		Function();
	}

	UE_FORCEINLINE_HINT static void DoTaskImpl(TUniqueFunction<void(const FGraphEventRef&)>& Function,
		ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		Function(MyCompletionGraphEvent);
	}

	UE_FORCEINLINE_HINT static void DoTaskImpl(TUniqueFunction<void(ENamedThreads::Type, const FGraphEventRef&)>& Function,
		ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		Function(CurrentThread, MyCompletionGraphEvent);
	}
};

struct FFunctionGraphTask
{
public:
	/**
	 * Create a task and dispatch it when the prerequisites are complete
	 * @param InFunction - a functor object to execute when the prerequisites are complete, with signature `void()` or `void(ENamedThreads::Type, const FGraphEventRef&)`
	 * @param InStatId - StatId of task for debugging or analysis tools
	 * @param InPrerequisites - Handles for prerequisites for this task, can be NULL if there are no prerequisites
	 * @param InDesiredThread - Thread to run on
	 * @return completion handle for the new task
	 **/
	static FGraphEventRef CreateAndDispatchWhenReady(TUniqueFunction<void()> InFunction, TStatId InStatId = TStatId{}, const FGraphEventArray* InPrerequisites = nullptr, ENamedThreads::Type InDesiredThread = ENamedThreads::AnyThread)
	{
		return TGraphTask<TFunctionGraphTaskImpl<void(), ESubsequentsMode::TrackSubsequents>>::CreateTask(InPrerequisites).ConstructAndDispatchWhenReady(MoveTemp(InFunction), InStatId, InDesiredThread);
	}

	static FGraphEventRef CreateAndDispatchWhenReady(TUniqueFunction<void(ENamedThreads::Type, const FGraphEventRef&)> InFunction, TStatId InStatId = TStatId{}, const FGraphEventArray* InPrerequisites = nullptr, ENamedThreads::Type InDesiredThread = ENamedThreads::AnyThread)
	{
		return TGraphTask<TFunctionGraphTaskImpl<void(ENamedThreads::Type, const FGraphEventRef&), ESubsequentsMode::TrackSubsequents>>::CreateTask(InPrerequisites).ConstructAndDispatchWhenReady(MoveTemp(InFunction), InStatId, InDesiredThread);
	}

	/**
	 * Create a task and dispatch it when the prerequisites are complete
	 * @param InFunction - a function to execute when the prerequisites are complete, with signature `void()` or `void(ENamedThreads::Type, const FGraphEventRef&)`
	 * @param InStatId - StatId of task for debugging or analysis tools
	 * @param InPrerequisite - Handle for a single prerequisite for this task
	 * @param InDesiredThread - Thread to run on
	 * @return completion handle for the new task
	 **/
	static FGraphEventRef CreateAndDispatchWhenReady(TUniqueFunction<void()> InFunction, TStatId InStatId, const FGraphEventRef& InPrerequisite, ENamedThreads::Type InDesiredThread = ENamedThreads::AnyThread)
	{
		FGraphEventArray Prerequisites;
		check(InPrerequisite.GetReference());
		Prerequisites.Add(InPrerequisite);
		return CreateAndDispatchWhenReady(MoveTemp(InFunction), InStatId, &Prerequisites, InDesiredThread);
	}

	static FGraphEventRef CreateAndDispatchWhenReady(TUniqueFunction<void(ENamedThreads::Type, const FGraphEventRef&)> InFunction, TStatId InStatId, const FGraphEventRef& InPrerequisite, ENamedThreads::Type InDesiredThread = ENamedThreads::AnyThread)
	{
		FGraphEventArray Prerequisites;
		check(InPrerequisite.GetReference());
		Prerequisites.Add(InPrerequisite);
		return CreateAndDispatchWhenReady(MoveTemp(InFunction), InStatId, &Prerequisites, InDesiredThread);
	}
};


