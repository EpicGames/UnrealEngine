// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Trace/Config.h"
#include "Trace/Trace.h"
#include "Trace/Detail/Channel.h"

namespace UE { namespace Trace { class FChannel; } }

#if !defined(UE_TASK_TRACE_ENABLED)
#if UE_TRACE_ENABLED && !IS_PROGRAM && !UE_BUILD_SHIPPING
#define UE_TASK_TRACE_ENABLED 1
#else
#define UE_TASK_TRACE_ENABLED 0
#endif
#endif

namespace ENamedThreads
{
	// Forward declare
	enum Type : int32;
}

#if UE_TASK_TRACE_ENABLED
#define TASK_CORE_API CORE_API
#else
#define TASK_CORE_API
#endif

namespace TaskTrace
{
	UE_TRACE_CHANNEL_EXTERN(TaskChannel, CORE_API);

	using FId = uint64;

	inline const FId InvalidId = ~FId(0);

	inline constexpr uint32 TaskTraceVersion = 1;

	FId TASK_CORE_API GenerateTaskId();

	void TASK_CORE_API Init();
	void TASK_CORE_API Created(FId TaskId, uint64 TaskSize); // optional, used only if a task was created but not launched immediately
	void TASK_CORE_API Launched(FId TaskId, const TCHAR* DebugName, bool bTracked, ENamedThreads::Type ThreadToExecuteOn, uint64 TaskSize);
	void TASK_CORE_API Scheduled(FId TaskId);
	void TASK_CORE_API SubsequentAdded(FId TaskId, FId SubsequentId);
	void TASK_CORE_API Started(FId TaskId);
	void TASK_CORE_API Finished(FId TaskId);
	void TASK_CORE_API Completed(FId TaskId);
	void TASK_CORE_API Destroyed(FId TaskId);

	struct FWaitingScope
	{
		TASK_CORE_API explicit FWaitingScope(const TArray<FId>& Tasks); // waiting for given tasks completion
		TASK_CORE_API explicit FWaitingScope(FId TaskId);
		TASK_CORE_API ~FWaitingScope();
	};

	struct FTaskTimingEventScope
	{
		TASK_CORE_API FTaskTimingEventScope(TaskTrace::FId InTaskId);
		TASK_CORE_API ~FTaskTimingEventScope();

	private:
		bool bIsActive = false;
		TaskTrace::FId TaskId = InvalidId;
	};

#if !UE_TASK_TRACE_ENABLED
	// NOOP implementation
	inline FId GenerateTaskId() { return InvalidId; }
	inline void Init() {}
	inline void Created(FId TaskId, uint64 TaskSize) {}
	inline void Launched(FId TaskId, const TCHAR* DebugName, bool bTracked, ENamedThreads::Type ThreadToExecuteOn, uint64 TaskSize) {}
	inline void Scheduled(FId TaskId) {}
	inline void SubsequentAdded(FId TaskId, FId SubsequentId) {}
	inline void Started(FId TaskId) {}
	inline void Finished(FId TaskId) {}
	inline void Completed(FId TaskId) {}
	inline void Destroyed(FId TaskId) {}
	inline FWaitingScope::FWaitingScope(const TArray<FId>& Tasks) {}
	inline FWaitingScope::FWaitingScope(FId TaskId) {}
	inline FWaitingScope::~FWaitingScope() {}
	inline FTaskTimingEventScope::FTaskTimingEventScope(TaskTrace::FId InTaskId) {}
	inline FTaskTimingEventScope::~FTaskTimingEventScope() {}
#endif // UE_TASK_TRACE_ENABLED
}
