// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "AutoRTFM.h"
#include "AutoRTFM/OpenWrapper.h"
#include "Templates/SharedPointer.h"

namespace UE
{

#if UE_AUTORTFM

// A transactionally safe mutex that works in the following novel ways:
// - In the open (non-transactional):
//   - Take the lock like before. Simple!
//   - Free the lock like before too.
// - In the closed (transactional):
//   - Unlock() decrements `TransactionalLockCount` and registers the on-commit handler (if not 
//     already registered) to perform the deferred unlock to the underlying mutex, if not rebalanced
//     in the transaction with a lock.
//   - During locking we query `TransactionalLockCount`:
//     * -1 means the transaction performed an Unlock() before the lock, so the underlying mutex 
//       must have been locked prior to the start of the transaction. `TransactionalLockCount` is
//       incremented to 0, and a lock is emulated (i.e. TryLock() will return `true` without
//       touching the underlying mutex).
//     * 0 means the underlying mutex will be locked if it hasn't been already (tracked by 
//       `bTransactionallyLocked`). Once the underlying mutex is locked, it remains locked for the
//       entire duration of the transaction. This is done as a transactional abort will likely undo
//       the writes to memory that is guarded by the mutex, so the lock needs be held until the
//       transaction is either fully committed or aborted. When the underlying mutex is locked an
//       on-abort handler and on-commit handler are registered to unlock the underlying mutex if
//       necessary.
//
// Thus with this approach we will hold this lock for the *entirety* of the transactional nest should
// we take the lock during the transaction, thus preventing non-transactional code from seeing any
// modifications we should make.
struct FTransactionallySafeMutex final
{
	FTransactionallySafeMutex() : State(MakeShared<FState>())
	{
	}

	// Construct in a locked state.
	inline explicit FTransactionallySafeMutex(UE::FAcquireLock Tag) : State(MakeShared<FState>(Tag))
	{
	}

	void Lock()
	{
		if (AutoRTFM::IsClosed() || AutoRTFM::IsCommittingOrAborting())
		{
			LockTransactionally();
		}
		else if (AutoRTFM::IsTransactional())
		{
			InTheClosed<&FTransactionallySafeMutex::LockTransactionally>();
		}
		else
		{
			// Non-transactional path. Call Lock() on the mutex directly.
			State->MutexLock();
		}
	}

	[[nodiscard]] inline bool TryLock()
	{
		if (AutoRTFM::IsClosed() || AutoRTFM::IsCommittingOrAborting())
		{
			return TryLockTransactionally();
		}
		else if (AutoRTFM::IsTransactional())
		{
			return InTheClosed<&FTransactionallySafeMutex::TryLockTransactionally>();
		}
		else
		{
			// Non-transactional path. Call TryLock() on the mutex directly.
			return State->Mutex.TryLock();
		}
	}

	void Unlock()
	{
		if (AutoRTFM::IsClosed() || AutoRTFM::IsCommittingOrAborting())
		{
			UnlockTransactionally();
		}
		else if (AutoRTFM::IsTransactional())
		{
			InTheClosed<&FTransactionallySafeMutex::UnlockTransactionally>();
		}
		else
		{
			State->MutexUnlock();
		}
	}

	/** 
	 * This method may give surprising results and should be used with caution!
	 * 
	 * - You cannot safely use this function to determine whether Lock() will block.
	 *   You may be in race with another thread which is also trying to lock the mutex.
	 * 
	 * - If a Mutex is locked by the AutoRTFM thread, it will not fully release the mutex
	 *   until the transaction fully succeeds or rolls back. Because of this, IsLocked may
	 *   return true even after the mutex has been fully unlocked.
	 */
	[[nodiscard]] bool IsLocked() const
	{
		return State->MutexIsLocked();
	}

private:
	UE_NONCOPYABLE(FTransactionallySafeMutex)

	template <bool (FTransactionallySafeMutex::*Functor)()>
	bool InTheClosed()
	{
		bool Result;
		AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([this, &Result]
		{
			Result = (this->*Functor)();
		});
		check(Status == AutoRTFM::ETransactionStatus::Executing);
		return Result;
	}

	bool TryLockTransactionally()
	{
		check(AutoRTFM::IsClosed() || AutoRTFM::IsCommittingOrAborting());
		ensure(State->TransactionalLockCount >= -1 && State->TransactionalLockCount <= 1);

		if (State->TransactionalLockCount > 0)
		{
			return false; // Attempting double-lock within transaction.
		}

		if (State->bTransactionallyLocked)
		{
			// Underlying mutex has already been locked for the duration of the transaction.
			State->TransactionalLockCount++;
			return true;
		}

		if (State->TransactionalLockCount < 0)
		{
			// The mutex was locked before the transaction and unlocked inside the transaction.
			// As unlock is deferred, bump the lock counter and return true to emulate the lock.
			State->TransactionalLockCount++;
			return true;
		}

		// Attempt to lock the underlying mutex.
		if (!State->MutexTryLock())
		{
			return false;
		}

		// First time the mutex has been locked during the transaction.
		// Increment the lock count, mark the transactionally locked flag and register 
		// on-commit and on-abort handlers.
		State->TransactionalLockCount++;
		SetTransactionallyLockedAndRegisterHandlers();
		return true;
	};

	// Acquires the mutex in a transactionally-safe way. Assumes we are either in the closed or
	// are mid-commit/mid-abort.
	bool LockTransactionally()
	{
		check(AutoRTFM::IsClosed() || AutoRTFM::IsCommittingOrAborting());
		ensure(State->TransactionalLockCount >= -1 && State->TransactionalLockCount <= 0);

		State->TransactionalLockCount++;

		if (State->bTransactionallyLocked)
		{
			// Underlying mutex has already been locked for the duration of the transaction.
			return true;
		}

		if (State->TransactionalLockCount == 0)
		{
			// The mutex was locked before the transaction and unlocked inside the transaction.
			return true;
		}

		// First time the mutex has been locked during the transaction.
		// Mark the transactionally locked flag and register on-commit and on-abort handlers.
		State->MutexLock();
		SetTransactionallyLockedAndRegisterHandlers();
		return true;
	}

	// Releases the mutex in a transactionally-safe way. Assumes we are either in the closed or
	// are mid-commit/mid-abort.
	bool UnlockTransactionally()
	{
		check(AutoRTFM::IsClosed() || AutoRTFM::IsCommittingOrAborting());
		ensure(State->TransactionalLockCount >= 0 && State->TransactionalLockCount <= 1);

		State->TransactionalLockCount--;
		// Use an on-commit handler to unlock the underlying mutex if not balanced with a lock
		// before the transaction commits.
		MaybeRegisterCommitHandler();
		return true;
	}

	// Called when the underlying mutex is locked for the first time within a transaction nest.
	// Sets the bTransactionallyLocked flag, calls MaybeRegisterCommitHandler() and registers an
	// on-abort handler to unlock the underlying mutex.
	void SetTransactionallyLockedAndRegisterHandlers()
	{
		ensure(!State->bTransactionallyLocked);
		State->bTransactionallyLocked = true;

		MaybeRegisterCommitHandler();

		// Capture State instead of 'this' as State is a TSharedPtr which can outlive this 
		// FTransactionallySafeMutex.
		AutoRTFM::OnAbort([State = AutoRTFM::TOpenWrapper{this->State}]
		{
			State.Object->MutexUnlock();
			State.Object->ResetTransactionState();
		});
	}
	
	// Registers an on-commit handler (if it hasn't been already registered for this transaction
	// nest) to unlock the underlying mutex if the transaction unlocked more times than locked.
	void MaybeRegisterCommitHandler()
	{
		if (!State->bRegisteredCommitHandler) 
		{
			State->bRegisteredCommitHandler = true;

			// Capture State instead of 'this' as State is a TSharedPtr which can outlive this 
			// FTransactionallySafeMutex.
			AutoRTFM::OnCommit([State = AutoRTFM::TOpenWrapper{this->State}]
			{
				// If the transaction's lock count is negative (more unlocks than locks), or the 
				// underlying mutex was locked and then re-balanced to an unlocked state, unlock
				// the underlying mutex.
				if (State.Object->TransactionalLockCount < 0 ||
					(State.Object->bTransactionallyLocked && State.Object->TransactionalLockCount == 0))
				{
					State.Object->MutexUnlock();
				}
				State.Object->ResetTransactionState();
			});
		}
	}

	struct FState final
	{
		FState() : bTransactionallyLocked(false), bRegisteredCommitHandler(false) {}

		explicit FState(UE::FAcquireLock Tag) : Mutex(Tag), bTransactionallyLocked(false), bRegisteredCommitHandler(false) {}

		AUTORTFM_DISABLE
		~FState()
		{
			ensure(TransactionalLockCount == 0);
			ensure(bTransactionallyLocked == false);
			ensure(bRegisteredCommitHandler == false);
		}

		AUTORTFM_DISABLE
		void ResetTransactionState()
		{
			TransactionalLockCount = 0;
			bTransactionallyLocked = false;
			bRegisteredCommitHandler = false;
		}

		AUTORTFM_OPEN
		void MutexLock()
		{
			Mutex.Lock();
		}
		
		AUTORTFM_OPEN
		bool MutexTryLock()
		{
			return Mutex.TryLock();
		}

		AUTORTFM_OPEN
		void MutexUnlock()
		{
			Mutex.Unlock();
		}

		AUTORTFM_OPEN
		bool MutexIsLocked()
		{
			return Mutex.IsLocked();
		}

		// The underlying mutex.
		UE::FMutex Mutex;
		// Counter for calls to unlock / lock within a transaction:
		// -1: mutex has had one more unlock than lock. (e.g. transaction starts locked)
		//  0: mutex has had the same number of locks as unlocks.
		// +1: mutex has had one more lock than unlock.
		// Other values indicate a double-lock or double-unlock within a transaction.
		int8 TransactionalLockCount = 0;
		// True if the underlying mutex has been locked for the duration of the transaction.
		bool bTransactionallyLocked : 1;
		// True if the on-commit handler has been registered for this transaction nest.
		bool bRegisteredCommitHandler : 1;
	};

	TSharedPtr<FState> State;
};

#else
using FTransactionallySafeMutex = FMutex;
#endif // UE_AUTORTFM

} // UE

// Some existing code references FTransactionallySafeMutex without the UE
// namespace prefix. TODO: properly qualify these and remove this.
using FTransactionallySafeMutex = UE::FTransactionallySafeMutex;
