// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AutoRTFM/Defines.h"
#include "HAL/Platform.h"

#define UE_API CORE_API

namespace UE::Private::TransactionallySafeMutexPool
{

// The size of an object in the allocator pool is 16 bytes. This value is chosen to match
// `sizeof(FTransactionallySafeRecursiveMutex::FState)`. The other transactionally-safe
// mutexes have state objects which also fit into 16 bytes or less.
constexpr size_t PooledObjectSize = 16;

/** Takes `PooledObjectSize` raw bytes from our mutex pool. */
UE_API void* Allocate();

/** Constructs a new state object, taking it from the pool. */
template <typename T>
	requires (sizeof(T) <= PooledObjectSize)
UE_REWRITE T* New()
{
	return new (UE::Private::TransactionallySafeMutexPool::Allocate()) T;
}

/** Returns `PooledObjectSize` raw bytes to the mutex pool. */
UE_API void Free(void* Ptr);

/** Destroys one state object, returning it to the pool. */
template <typename T>
	requires (sizeof(T) <= PooledObjectSize)
UE_REWRITE void Delete(T* Ptr)
{
	Ptr->~T();
	UE::Private::TransactionallySafeMutexPool::Free(Ptr);
}

#if !UE_BUILD_SHIPPING
/** Returns the number of objects currently allocated from the pool. Intended for testing. */
UE_API int GetNumUsed();
#endif

} // namespace UE::Private::TransactionallySafeMutexPool

#undef UE_API
