// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/SharedMutex.h"
#include "Async/TransactionallySafeRecursiveMutex.h"

namespace UE
{

#if UE_AUTORTFM
using FTransactionallySafeSharedMutex = ::UE::TTransactionallySafeMutex<::UE::FSharedMutex>;
#else
using FTransactionallySafeSharedMutex = ::UE::FSharedMutex;
#endif

}  // namespace UE
