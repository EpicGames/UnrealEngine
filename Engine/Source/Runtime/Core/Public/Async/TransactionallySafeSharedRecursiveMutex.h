// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/SharedRecursiveMutex.h"
#include "AutoRTFM.h"

#if UE_AUTORTFM
#include "Async/TransactionallySafeRecursiveMutex.h"
#endif

namespace UE
{

#if UE_AUTORTFM

using FTransactionallySafeSharedRecursiveMutex = ::UE::TTransactionallySafeMutex<::UE::FSharedRecursiveMutex>;

template <>
class TSharedLock<FTransactionallySafeSharedRecursiveMutex> final
{
public:
	TSharedLock(const TSharedLock&) = delete;
	TSharedLock& operator=(const TSharedLock&) = delete;

	[[nodiscard]] inline explicit TSharedLock(FTransactionallySafeSharedRecursiveMutex& Lock)
		: Mutex(Lock)
	{
		Mutex.LockShared(Link);
	}

	inline ~TSharedLock()
	{
		Mutex.UnlockShared(Link);
	}

private:
	FTransactionallySafeSharedRecursiveMutex& Mutex;
	Core::Private::FSharedRecursiveMutexLink Link;
};

#else
using FTransactionallySafeSharedRecursiveMutex = ::UE::FSharedRecursiveMutex;
#endif

}  // namespace UE
