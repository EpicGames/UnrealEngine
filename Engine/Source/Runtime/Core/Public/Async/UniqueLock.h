// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/LockTags.h"
#include "Misc/AssertionMacros.h"

namespace UE
{

/**
 * A basic mutex ownership wrapper that locks on construction and unlocks on destruction.
 *
 * LockType must contain Lock() and Unlock() functions.
 * 
 * Use with mutex types like FMutex and FRecursiveMutex.
 */
template <typename LockType>
class TUniqueLock final
{
public:
	TUniqueLock(const TUniqueLock&) = delete;
	TUniqueLock& operator=(const TUniqueLock&) = delete;

	[[nodiscard]] inline explicit TUniqueLock(LockType& Lock)
		: Mutex(Lock)
	{
		Mutex.Lock();
	}

	inline ~TUniqueLock()
	{
		Mutex.Unlock();
	}

private:
	LockType& Mutex;
};

/**
 * A mutex ownership wrapper that allows dynamic locking, unlocking, and deferred locking.
 *
 * LockType must contain Lock() and Unlock() functions.
 * 
 * Use with mutex types like FMutex and FRecursiveMutex.
 */
template <typename LockType>
class TDynamicUniqueLock final
{
public:
	TDynamicUniqueLock() = default;

	TDynamicUniqueLock(const TDynamicUniqueLock&) = delete;
	TDynamicUniqueLock& operator=(const TDynamicUniqueLock&) = delete;

	/** Wrap a mutex and lock it. */
	[[nodiscard]] inline explicit TDynamicUniqueLock(LockType& Lock)
		: Mutex(&Lock)
	{
		Mutex->Lock();
		bLocked = true;
	}

	/** Wrap a mutex without locking it. */
	[[nodiscard]] inline explicit TDynamicUniqueLock(LockType& Lock, FDeferLock)
		: Mutex(&Lock)
	{
	}

	/** Move from another lock, transferring any ownership to this lock. */
	[[nodiscard]] inline TDynamicUniqueLock(TDynamicUniqueLock&& Other)
		: Mutex(Other.Mutex)
		, bLocked(Other.bLocked)
	{
		Other.Mutex = nullptr;
		Other.bLocked = false;
	}

	/** Move from another lock, transferring any ownership to this lock, and unlocking the previous mutex if locked. */
	inline TDynamicUniqueLock& operator=(TDynamicUniqueLock&& Other)
	{
		if (bLocked)
		{
			Mutex->Unlock();
		}
		Mutex = Other.Mutex;
		bLocked = Other.bLocked;
		Other.Mutex = nullptr;
		Other.bLocked = false;
		return *this;
	}

	/** Unlock the mutex if locked. */
	inline ~TDynamicUniqueLock()
	{
		if (bLocked)
		{
			Mutex->Unlock();
		}
	}

	/** Lock the associated mutex. This lock must have a mutex and must not be locked. */
	void Lock()
	{
		check(!bLocked);
		check(Mutex);
		Mutex->Lock();
		bLocked = true;
	}

	/** Unlock the associated mutex. This lock must have a mutex and must be locked. */
	void Unlock()
	{
		check(bLocked);
		bLocked = false;
		Mutex->Unlock();
	}

	/** Returns true if this lock has its associated mutex locked. */
	inline bool OwnsLock() const
	{
		return bLocked;
	}

	/** Returns true if this lock has its associated mutex locked. */
	inline explicit operator bool() const
	{
		return OwnsLock();
	}

private:
	LockType* Mutex = nullptr;
	bool bLocked = false;
};

} // UE
