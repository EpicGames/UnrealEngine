// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include <atomic>

#define UE_API CORE_API

namespace UE
{

/**
 * A mutex that is the size of a pointer and does not depend on ParkingLot.
 *
 * Prefer FMutex to FWordMutex whenever possible.
 * This mutex is not fair and does not support recursive locking.
 */
class FWordMutex final
{
public:
	constexpr FWordMutex() = default;

	FWordMutex(const FWordMutex&) = delete;
	FWordMutex& operator=(const FWordMutex&) = delete;

	[[nodiscard]] inline bool IsLocked() const
	{
		return !!(State.load(std::memory_order_relaxed) & IsLockedFlag);
	}

	[[nodiscard]] inline bool TryLock()
	{
		UPTRINT Expected = 0;
		return State.compare_exchange_strong(Expected, IsLockedFlag, std::memory_order_acquire, std::memory_order_relaxed);
	}

	inline void Lock()
	{
		UPTRINT Expected = 0;
		if (LIKELY(State.compare_exchange_weak(Expected, IsLockedFlag, std::memory_order_acquire, std::memory_order_relaxed)))
		{
			return;
		}

		LockSlow();
	}

	inline void Unlock()
	{
		// Unlock immediately to allow other threads to acquire the lock while this thread looks for a thread to wake.
		UPTRINT CurrentState = State.fetch_sub(IsLockedFlag, std::memory_order_release);
		checkSlow(CurrentState & IsLockedFlag);

		// An empty queue indicates that there are no threads to wake.
		const bool bQueueEmpty = !(CurrentState & QueueMask);
		// A locked queue indicates that another thread is looking for a thread to wake.
		const bool bQueueLocked = (CurrentState & IsQueueLockedFlag);

		if (LIKELY(bQueueEmpty || bQueueLocked))
		{
			return;
		}

		UnlockSlow(CurrentState);
	}

private:
	UE_API void LockSlow();
	UE_API void UnlockSlow(UPTRINT CurrentState);

	static constexpr UPTRINT IsLockedFlag = 1 << 0;
	static constexpr UPTRINT IsQueueLockedFlag = 1 << 1;
	static constexpr UPTRINT QueueMask = ~(IsLockedFlag | IsQueueLockedFlag);

	std::atomic<UPTRINT> State = 0;
};

} // UE

#undef UE_API
