// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

class FArchive;

namespace Audio
{
	/**
	 * SimpleWaveReader class
	 *
	 * Only able to read wave files of a specific format
	 * Specifically, it should be able to read any wave file written via the SimpleWaveWriter class
	 */
	class FSimpleWaveReader
	{
	public:
		CORE_API FSimpleWaveReader(TUniquePtr<FArchive>&& InOutputStream);
		CORE_API ~FSimpleWaveReader();

		CORE_API void SeekToFrame(uint32 FrameIndex) const;

		// returns true if the end of the file was reached
		CORE_API bool Read(TArrayView<const float> OutBuffer, int64& OutNumSamplesRead) const;

		CORE_API bool IsDataValid() const;

		CORE_API uint32 GetSampleRate() const;

		CORE_API uint16 GetNumChannels() const;

		CORE_API uint32 GetNumSamples() const;
		
	private:
	
		bool ReadHeader(uint32& OutSampleRate, uint16& OutNumChannels, uint32& OutDataSize);
		
		TUniquePtr<FArchive> InputStream;
		int64 DataStartPos = 0;

		uint32 SampleRate = 0;
		uint16 NumChannels = 0;
		uint32 DataSize = 0;
		bool bIsDataValid = false;
	};
}