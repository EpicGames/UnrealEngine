// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"

namespace Audio
{
	class FSimpleWaveWriter
	{
	public:
		CORE_API FSimpleWaveWriter(TUniquePtr<FArchive>&& InOutputStream, int32 InSampleRate, int32 InNumChannels, bool bInUpdateHeaderAfterEveryWrite);
		CORE_API ~FSimpleWaveWriter();

		CORE_API void Write(TArrayView<const float> InBuffer);

	private:
		void WriteHeader(int32 InSampleRate, int32 InNumChannels);
		void UpdateHeader();

		TUniquePtr<FArchive> OutputStream;
		int32 RiffSizePos = 0;
		int32 DataSizePos = 0;
		bool bUpdateHeaderAfterEveryWrite = false;
	};
}