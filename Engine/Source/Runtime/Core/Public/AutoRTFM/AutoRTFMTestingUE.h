// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/Testing.h"
#include "Templates/Function.h"


namespace AutoRTFM::Testing
{

#if AUTORTFM_ENABLE_TEST_UTILS

// Calls Intercept to modify the autortfm_extern_api before calling
// autortfm_initialize(), so that tests can intercept or customize the extern
// APIs.
AUTORTFM_DISABLE CORE_API void InterceptExternAPIs(TFunction<void(autortfm_extern_api&)> Intercept);

// Installs a crash handler to print the callstack if the process crashes.
// Can only be called once for the process.
AUTORTFM_DISABLE CORE_API void PrintCallstackOnCrash();

#else // ^^^ AUTORTFM_ENABLE_TEST_UTILS ^^^ | vvv !AUTORTFM_ENABLE_TEST_UTILS vvv

inline void InterceptExternAPIs([[maybe_unused]] TFunction<void(autortfm_extern_api&)> Intercept) {}

inline void PrintCallstackOnCrash() {}

#endif // ^^^ !AUTORTFM_ENABLE_TEST_UTILS ^^^
}

