// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM.h"
#include "Logging/LogMacros.h"

CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogAutoRTFM, Display, All)

namespace AutoRTFM
{

using RollbackWriteFn = void (*)(void* Address, const void* Value, size_t Size, autortfm_write_flags Flags);

#if UE_AUTORTFM

// Initializes AutoRTFM for use by the Unreal Engine.
// RollbackWrite is an optional function pointer used to rollback writes written
// with the AutoRTFM::EWriteFlags::CustomRollback flag.
CORE_API void InitializeForUE(RollbackWriteFn RollbackWrite = nullptr);

#else

inline void InitializeForUE([[maybe_unused]] RollbackWriteFn RollbackWrite = nullptr) {}

#endif

}
