// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	ClangPlatform.h: Setup for any Clang-using platform
==================================================================================*/

#pragma once

// HEADER_UNIT_UNSUPPORTED - Clang not supporting header units

#if defined(__clang__)

#if !defined(__cpp_if_constexpr)
	#error "Compiler is expected to support if constexpr"
#endif

#if !defined(__cpp_fold_expressions)
	#error "Compiler is expected to support fold expressions"
#endif

#if !__has_feature(cxx_decltype_auto)
	#error "Compiler is expected to support decltype(auto)"
#endif

#define PLATFORM_RETURN_ADDRESS()			__builtin_return_address(0)
#define PLATFORM_RETURN_ADDRESS_POINTER()	__builtin_frame_address(0)

#define UE_LIFETIMEBOUND [[clang::lifetimebound]]

#define UE_NODEBUG [[gnu::nodebug]]

// Clang supports the following GNU attributes to let us describe allocation functions.
// `gnu::malloc` maps to the `noalias` function return on the caller in the LLVM IR.
// `gnu::alloc_size` maps to the `allocsize` function attribute on the caller in the LLVM IR.
// `gnu::alloc_align` will cause any memory operations in the callee to have the specified alignment in the LLVM IR.
// We use a selector macro trick to map up to 2 inputs to the correct macros.
#define UE_ALLOCATION_FUNCTION_0() [[gnu::malloc]]
#define UE_ALLOCATION_FUNCTION_1(SIZE) [[gnu::malloc, gnu::alloc_size(SIZE)]]

// Note: we cannot use gnu::alloc_align(ALIGN) here yet, because the DEFAULT_ALIGNMENT argument is 0, which is not
//       a power of two, and Clang will complain about uses of a function with a default alignment that is not a
//       power of two.
#define UE_ALLOCATION_FUNCTION_2(SIZE, ALIGN) [[gnu::malloc, gnu::alloc_size(SIZE)]]
#define UE_ALLOCATION_FUNCTION_X(x, SIZE, ALIGN, FUNC, ...) FUNC
#define UE_ALLOCATION_FUNCTION(...) UE_ALLOCATION_FUNCTION_X(,##__VA_ARGS__, UE_ALLOCATION_FUNCTION_2(__VA_ARGS__), UE_ALLOCATION_FUNCTION_1(__VA_ARGS__), UE_ALLOCATION_FUNCTION_0(__VA_ARGS__)) 

// Ensure we can use this builtin - seems to be present on Clang 9, GCC 11 and MSVC 19.26,
#define PLATFORM_COMPILER_SUPPORTS_BUILTIN_BITCAST (__clang_major__ >= 9)

#define FUNCTION_NON_NULL_RETURN_START [[gnu::returns_nonnull]]

#if defined(__has_attribute)
#	if __has_attribute(no_profile_instrument_function)
#		define UE_NO_PROFILE_ATTRIBUTE __attribute__((no_profile_instrument_function))
#	endif
#endif

#ifndef UE_NO_PROFILE_ATTRIBUTE
#	define UE_NO_PROFILE_ATTRIBUTE
#endif

#define CALLSITE_FORCEINLINE [[clang::always_inline]]			/* Force code to be inlined at the callsite */

#endif//defined(__clang__)
