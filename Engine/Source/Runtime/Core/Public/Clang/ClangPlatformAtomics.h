// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	ClangPlatformAtomics.h: Clang Atomics functions
==============================================================================================*/

#pragma once

#ifdef __clang__

#include "GenericPlatform/GenericPlatformAtomics.h"
#include "CoreTypes.h"

// HEADER_UNIT_UNSUPPORTED - Clang not supporting header units

/**
 * GCC/Clang implementation of the Atomics OS functions
 **/
struct FClangPlatformAtomics : public FGenericPlatformAtomics
{
	static FORCEINLINE int8 InterlockedIncrement(volatile int8* Value)
	{
		return __atomic_fetch_add(Value, 1, __ATOMIC_SEQ_CST) + 1;
	}

	static FORCEINLINE int16 InterlockedIncrement(volatile int16* Value)
	{
		return __atomic_fetch_add(Value, 1, __ATOMIC_SEQ_CST) + 1;
	}

	static FORCEINLINE int32 InterlockedIncrement(volatile int32* Value)
	{
		return __atomic_fetch_add(Value, 1, __ATOMIC_SEQ_CST) + 1;
	}

	static FORCEINLINE int64 InterlockedIncrement(volatile int64* Value)
	{
		return __atomic_fetch_add(Value, 1, __ATOMIC_SEQ_CST) + 1;
	}

	static FORCEINLINE int8 InterlockedDecrement(volatile int8* Value)
	{
		return __atomic_fetch_sub(Value, 1, __ATOMIC_SEQ_CST) - 1;
	}

	static FORCEINLINE int16 InterlockedDecrement(volatile int16* Value)
	{
		return __atomic_fetch_sub(Value, 1, __ATOMIC_SEQ_CST) - 1;
	}

	static FORCEINLINE int32 InterlockedDecrement(volatile int32* Value)
	{
		return __atomic_fetch_sub(Value, 1, __ATOMIC_SEQ_CST) - 1;
	}

	static FORCEINLINE int64 InterlockedDecrement(volatile int64* Value)
	{
		return __atomic_fetch_sub(Value, 1, __ATOMIC_SEQ_CST) - 1;
	}

	static FORCEINLINE int8 InterlockedAdd(volatile int8* Value, int8 Amount)
	{
		return __atomic_fetch_add(Value, Amount, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int16 InterlockedAdd(volatile int16* Value, int16 Amount)
	{
		return __atomic_fetch_add(Value, Amount, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int32 InterlockedAdd(volatile int32* Value, int32 Amount)
	{
		return __atomic_fetch_add(Value, Amount, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int64 InterlockedAdd(volatile int64* Value, int64 Amount)
	{
		return __atomic_fetch_add(Value, Amount, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int8 InterlockedExchange(volatile int8* Value, int8 Exchange)
	{
		return __atomic_exchange_n(Value, Exchange, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int16 InterlockedExchange(volatile int16* Value, int16 Exchange)
	{
		return __atomic_exchange_n(Value, Exchange, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int32 InterlockedExchange(volatile int32* Value, int32 Exchange)
	{
		return __atomic_exchange_n(Value, Exchange, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int64 InterlockedExchange(volatile int64* Value, int64 Exchange)
	{
		return __atomic_exchange_n(Value, Exchange, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE void* InterlockedExchangePtr(void*volatile* Dest, void* Exchange)
	{
		return __atomic_exchange_n(Dest, Exchange, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int8 InterlockedCompareExchange(volatile int8* Dest, int8 Exchange, int8 Comparand)
	{
		__atomic_compare_exchange_n(Dest, &Comparand, Exchange, false, __ATOMIC_SEQ_CST, __ATOMIC_SEQ_CST);
		return Comparand;
	}

	static FORCEINLINE int16 InterlockedCompareExchange(volatile int16* Dest, int16 Exchange, int16 Comparand)
	{
		__atomic_compare_exchange_n(Dest, &Comparand, Exchange, false, __ATOMIC_SEQ_CST, __ATOMIC_SEQ_CST);
		return Comparand;
	}

	static FORCEINLINE int32 InterlockedCompareExchange(volatile int32* Dest, int32 Exchange, int32 Comparand)
	{
		__atomic_compare_exchange_n(Dest, &Comparand, Exchange, false, __ATOMIC_SEQ_CST, __ATOMIC_SEQ_CST);
		return Comparand;
	}

	static FORCEINLINE int64 InterlockedCompareExchange(volatile int64* Dest, int64 Exchange, int64 Comparand)
	{
		__atomic_compare_exchange_n(Dest, &Comparand, Exchange, false, __ATOMIC_SEQ_CST, __ATOMIC_SEQ_CST);
		return Comparand;
	}

	static FORCEINLINE int8 InterlockedAnd(volatile int8* Value, const int8 AndValue)
	{
		return __atomic_fetch_and(Value, AndValue, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int16 InterlockedAnd(volatile int16* Value, const int16 AndValue)
	{
		return __atomic_fetch_and(Value, AndValue, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int32 InterlockedAnd(volatile int32* Value, const int32 AndValue)
	{
		return __atomic_fetch_and(Value, AndValue, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int64 InterlockedAnd(volatile int64* Value, const int64 AndValue)
	{
		return __atomic_fetch_and(Value, AndValue, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int8 InterlockedOr(volatile int8* Value, const int8 OrValue)
	{
		return __atomic_fetch_or(Value, OrValue, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int16 InterlockedOr(volatile int16* Value, const int16 OrValue)
	{
		return __atomic_fetch_or(Value, OrValue, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int32 InterlockedOr(volatile int32* Value, const int32 OrValue)
	{
		return __atomic_fetch_or(Value, OrValue, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int64 InterlockedOr(volatile int64* Value, const int64 OrValue)
	{
		return __atomic_fetch_or(Value, OrValue, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int8 InterlockedXor(volatile int8* Value, const int8 XorValue)
	{
		return __atomic_fetch_xor(Value, XorValue, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int16 InterlockedXor(volatile int16* Value, const int16 XorValue)
	{
		return __atomic_fetch_xor(Value, XorValue, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int32 InterlockedXor(volatile int32* Value, const int32 XorValue)
	{
		return __atomic_fetch_xor(Value, XorValue, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int64 InterlockedXor(volatile int64* Value, const int64 XorValue)
	{
		return __atomic_fetch_xor(Value, XorValue, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int8 AtomicRead(volatile const int8* Src)
	{
		return __atomic_load_n(Src, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int16 AtomicRead(volatile const int16* Src)
	{
		return __atomic_load_n(Src, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int32 AtomicRead(volatile const int32* Src)
	{
		return __atomic_load_n(Src, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int64 AtomicRead(volatile const int64* Src)
	{
		return __atomic_load_n(Src, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE int8 AtomicRead_Relaxed(volatile const int8* Src)
	{
		return __atomic_load_n(Src, __ATOMIC_RELAXED);
	}

	static FORCEINLINE int16 AtomicRead_Relaxed(volatile const int16* Src)
	{
		return __atomic_load_n(Src, __ATOMIC_RELAXED);
	}

	static FORCEINLINE int32 AtomicRead_Relaxed(volatile const int32* Src)
	{
		return __atomic_load_n(Src, __ATOMIC_RELAXED);
	}

	static FORCEINLINE int64 AtomicRead_Relaxed(volatile const int64* Src)
	{
		return __atomic_load_n(Src, __ATOMIC_RELAXED);
	}

	static FORCEINLINE void AtomicStore(volatile int8* Src, int8 Val)
	{
		__atomic_store_n((volatile int8*)Src, Val, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE void AtomicStore(volatile int16* Src, int16 Val)
	{
		__atomic_store_n((volatile int16*)Src, Val, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE void AtomicStore(volatile int32* Src, int32 Val)
	{
		__atomic_store_n((volatile int32*)Src, Val, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE void AtomicStore(volatile int64* Src, int64 Val)
	{
		__atomic_store_n((volatile int64*)Src, Val, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE void AtomicStore_Relaxed(volatile int8* Src, int8 Val)
	{
		__atomic_store_n((volatile int8*)Src, Val, __ATOMIC_RELAXED);
	}

	static FORCEINLINE void AtomicStore_Relaxed(volatile int16* Src, int16 Val)
	{
		__atomic_store_n((volatile int16*)Src, Val, __ATOMIC_RELAXED);
	}

	static FORCEINLINE void AtomicStore_Relaxed(volatile int32* Src, int32 Val)
	{
		__atomic_store_n((volatile int32*)Src, Val, __ATOMIC_RELAXED);
	}

	static FORCEINLINE void AtomicStore_Relaxed(volatile int64* Src, int64 Val)
	{
		__atomic_store_n((volatile int64*)Src, Val, __ATOMIC_RELAXED);
	}

	static FORCEINLINE void* InterlockedCompareExchangePointer(void*volatile* Dest, void* Exchange, void* Comparand)
	{
		__atomic_compare_exchange_n(Dest, &Comparand, Exchange, false, __ATOMIC_SEQ_CST, __ATOMIC_SEQ_CST);
		return Comparand;
	}

#if PLATFORM_HAS_128BIT_ATOMICS
	/**
	 *	The function compares the Destination value with the Comparand value:
	 *		- If the Destination value is equal to the Comparand value, the Exchange value is stored in the address specified by Destination,
	 *		- Otherwise, the initial value of the Destination parameter is stored in the address specified specified by Comparand.
	 *
	 *	@return true if Comparand equals the original value of the Destination parameter, or false if Comparand does not equal the original value of the Destination parameter.
	 *
	 *	Early AMD64 processors lacked the CMPXCHG16B instruction.
	 *	To determine whether the processor supports this operation, call the IsProcessorFeaturePresent function with PF_COMPARE64_EXCHANGE128.
	 */
	static FORCEINLINE bool InterlockedCompareExchange128(volatile FInt128* Dest, const FInt128& Exchange, FInt128* Comparand)
	{
		return __atomic_compare_exchange_n((volatile __int128*)Dest, (__int128*)Comparand, *(__int128*)&Exchange, false, __ATOMIC_SEQ_CST, __ATOMIC_SEQ_CST);
	}

	static FORCEINLINE void AtomicRead128(const FInt128* Src, FInt128* OutResult)
	{
		__atomic_load(Src, OutResult, __ATOMIC_SEQ_CST);
	}
#endif

	static FORCEINLINE bool CanUseCompareExchange128()
	{
		return !!PLATFORM_HAS_128BIT_ATOMICS;
	}
};

#endif
