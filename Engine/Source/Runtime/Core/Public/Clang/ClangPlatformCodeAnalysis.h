// Copyright Epic Games, Inc. All Rights Reserved.
// IWYU pragma: begin_exports

#pragma once

// HEADER_UNIT_UNSUPPORTED - Clang not supporting header units

// Guard to prevent non-unity incremental checks using MSVC to try to compile this header.
#if defined(__clang__)

// Code analysis features
#if defined( __clang_analyzer__ ) || defined( PVS_STUDIO )
	#define USING_CODE_ANALYSIS 1
#else
	#define USING_CODE_ANALYSIS 0
#endif

//
// NOTE: To suppress a single occurrence of a code analysis warning:
//
// 		CA_SUPPRESS( <WarningNumber> )
//		...code that triggers warning...
//

//
// NOTE: To disable all code analysis warnings for a section of code (such as include statements
//       for a third party library), you can use the following:
//
// 		#if USING_CODE_ANALYSIS
// 			MSVC_PRAGMA( warning( push ) )
// 			MSVC_PRAGMA( warning( disable : ALL_CODE_ANALYSIS_WARNINGS ) )
// 		#endif	// USING_CODE_ANALYSIS
//
//		<code with warnings>
//
// 		#if USING_CODE_ANALYSIS
// 			MSVC_PRAGMA( warning( pop ) )
// 		#endif	// USING_CODE_ANALYSIS
//

#if USING_CODE_ANALYSIS

	// If enabled, add in settings for PVS-Studio Analysis
	#include "Analysis/PvsStudioCodeAnalysis.h"

	// A fake function marked with noreturn that acts as a marker for CA_ASSUME to ensure the
	// static analyzer doesn't take an analysis path that is assumed not to be navigable.
	void CA_AssumeNoReturn() __attribute__((analyzer_noreturn));

	// Input argument
	// Example:  void SetValue( CA_IN bool bReadable );
	#define CA_IN

	// Output argument
	// Example:  void FillValue( CA_OUT bool& bWriteable );
	#define CA_OUT

	// Specifies that a function parameter may only be read from, never written.
	// NOTE: CA_READ_ONLY is inferred automatically if your parameter is has a const qualifier.
	// Example:  void SetValue( CA_READ_ONLY bool bReadable );
	#define CA_READ_ONLY

	// Specifies that a function parameter may only be written to, never read.
	// Example:  void FillValue( CA_WRITE_ONLY bool& bWriteable );
	#define CA_WRITE_ONLY

	// Incoming pointer parameter must not be NULL and must point to a valid location in memory.
	// Place before a function parameter's type name.
	// Example:  void SetPointer( CA_VALID_POINTER void* Pointer );
	#define CA_VALID_POINTER

	// Caller must check the return value.  Place before the return value in a function declaration.
	// Example:  CA_CHECK_RETVAL int32 GetNumber();
	#define CA_CHECK_RETVAL

	// Function is expected to never return
	#define CA_NO_RETURN __attribute__((analyzer_noreturn))

	// Suppresses a warning for a single occurrence.  Should be used only for code analysis warnings on Windows platform!
	#define CA_SUPPRESS( WarningNumber )

	// Tells the code analysis engine to assume the statement to be true.  Useful for suppressing false positive warnings.
	#define CA_ASSUME( Expr )  (__builtin_expect(!bool(Expr), 0) ? CA_AssumeNoReturn() : (void)0)

	// Does a simple 'if (Condition)', but disables warnings about using constants in the condition.  Helps with some macro expansions.
	#define CA_CONSTANT_IF(Condition) if (Condition)


	//
	// Disable some code analysis warnings that we are NEVER interested in
	//

	// NOTE: Please be conservative about adding new suppressions here!  If you add a suppression, please
	//       add a comment that explains the rationale.

#endif

#if defined(__has_feature) && __has_feature(address_sanitizer)
	#define USING_ADDRESS_SANITISER 1
#else
	#define USING_ADDRESS_SANITISER 0
#endif

#if defined(__has_feature) && __has_feature(hwaddress_sanitizer)
	#define USING_HW_ADDRESS_SANITISER 1
#else
	#define USING_HW_ADDRESS_SANITISER 0
#endif

#if defined(__has_feature) && __has_feature(thread_sanitizer)
	#define USING_THREAD_SANITISER 1
#else
	#define USING_THREAD_SANITISER 0
#endif

#if defined(__has_feature) && __has_feature(undefined_behavior_sanitizer)
	#define USING_UNDEFINED_BEHAVIOR_SANITISER 1
#else
	#define USING_UNDEFINED_BEHAVIOR_SANITISER 0
#endif

#if defined(__has_feature) && __has_feature(memory_sanitizer)
	#define USING_MEMORY_SANITISER 1
#else
	#define USING_MEMORY_SANITISER 0
#endif

#if USING_UNDEFINED_BEHAVIOR_SANITISER
	#define UE_DISABLE_UBSAN [[clang::no_sanitize("undefined")]]
#endif

#ifndef USING_INSTRUMENTATION
	#define USING_INSTRUMENTATION 0
#endif

#if USING_INSTRUMENTATION
	// For instrumentation purpose, it is important that we do not skip any memory ops
	// even if marked as TSAN_SAFE.
	#define TSAN_SAFE
#endif

// We do want all the additional annotations for the instrumentation too.
#if USING_THREAD_SANITISER || USING_INSTRUMENTATION

	// This might already have been defined in the instrumentation block above
#ifndef TSAN_SAFE
	// Function attribute to disable thread sanitiser validation on specific functions that assume non-atomic load/stores are implicitly atomic
	// This is only safe for int32/uint32/int64/uint64/uintptr_t/intptr_t/void* types on x86/x64 strongly-consistent memory systems.
	#define TSAN_SAFE __attribute__((no_sanitize("thread")))
#endif

	// Thread-sanitiser annotation functions.
	#if defined(__AUTORTFM) && __AUTORTFM
		// both open and closed code call directly, no closed variant or trampoline generated.
		#include "AutoRTFM/Constants.h"
		#define ANNOTATE_FUNC_ATTRIBUTES __attribute__((autortfm(autortfm_mode_internal)))
	#else
		#define ANNOTATE_FUNC_ATTRIBUTES
	#endif
	#ifdef __cplusplus
	extern "C" {
	#endif
		ANNOTATE_FUNC_ATTRIBUTES void AnnotateHappensBefore(const char *f, int l, void *addr);
		ANNOTATE_FUNC_ATTRIBUTES void AnnotateHappensAfter(const char *f, int l, void *addr);
	#ifdef __cplusplus
	}
	#endif
	#undef ANNOTATE_FUNC_ATTRIBUTES

	// Annotate that previous load/stores occur before addr 
	#define TSAN_BEFORE(addr) AnnotateHappensBefore(__FILE__, __LINE__, (void*)(addr))

	// Annotate that previous load/stores occur after addr 
	#define TSAN_AFTER(addr) AnnotateHappensAfter(__FILE__, __LINE__, (void*)(addr))

	// Because annotating the global bools is tiresome...
	#ifdef __cplusplus
	    #include <atomic>

		// We need something that is defaulting to relaxed to avoid introducing additional barrier
		// that cause issues to go unnoticed.
		template <typename T>
		class TTSANSafeValue
		{
			std::atomic<T> Value;

		public:
			__attribute__((always_inline)) TTSANSafeValue() = default;
			__attribute__((always_inline)) TTSANSafeValue(T InValue) : Value(InValue) { }
			__attribute__((always_inline)) operator T() const { return Value.load(std::memory_order_relaxed); }
			__attribute__((always_inline)) void operator=(T InValue) { Value.store(InValue, std::memory_order_relaxed); }
			__attribute__((always_inline)) T operator++() { return Value.fetch_add(1, std::memory_order_relaxed) + 1; }
			__attribute__((always_inline)) T operator++(int) { return Value.fetch_add(1, std::memory_order_relaxed); }
			__attribute__((always_inline)) T operator--() { return Value.fetch_sub(1, std::memory_order_relaxed) - 1; }
			__attribute__((always_inline)) T operator--(int) { return Value.fetch_sub(1, std::memory_order_relaxed); }

			__attribute__((always_inline)) T operator->() { return Value.load(std::memory_order_relaxed); }

			__attribute__((always_inline)) TTSANSafeValue(const TTSANSafeValue& Other)
			{
				Value.store((T)Other, std::memory_order_relaxed);
			}

			__attribute__((always_inline)) TTSANSafeValue& operator=(const TTSANSafeValue& Other)
			{
				Value.store((T)Other, std::memory_order_relaxed);
				return *this;
			}

			template <typename OtherType>
			__attribute__((always_inline)) T operator+=(OtherType InValue) { return Value.fetch_add(InValue, std::memory_order_relaxed) + InValue; }

			template <typename OtherType>
			__attribute__((always_inline)) T operator-=(OtherType InValue) { return Value.fetch_sub(InValue, std::memory_order_relaxed) - InValue; }
		};

	    #define TSAN_ATOMIC(Type) TTSANSafeValue<Type>
	#endif

#endif

#endif // __clang__

// IWYU pragma: end_exports
