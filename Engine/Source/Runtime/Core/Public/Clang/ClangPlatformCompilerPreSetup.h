// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_UNSUPPORTED - Clang not supporting header units

#ifdef __clang__

#ifndef UE_CLANG_SUPPRESS
	#define UE_CLANG_SUPPRESS [[clang::suppress]]
#endif

#pragma clang diagnostic warning "-Wdeprecated-declarations"

#ifndef PRAGMA_DISABLE_DEPRECATION_WARNINGS
	#define PRAGMA_DISABLE_DEPRECATION_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wdeprecated-declarations\"")
#endif // PRAGMA_DISABLE_DEPRECATION_WARNINGS

#ifndef PRAGMA_ENABLE_DEPRECATION_WARNINGS
	#define PRAGMA_ENABLE_DEPRECATION_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_DEPRECATION_WARNINGS

#ifndef PRAGMA_DISABLE_OVERLOADED_VIRTUAL_WARNINGS
	#define PRAGMA_DISABLE_OVERLOADED_VIRTUAL_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Woverloaded-virtual\"")
#endif // PRAGMA_DISABLE_OVERLOADED_VIRTUAL_WARNINGS

#ifndef PRAGMA_ENABLE_OVERLOADED_VIRTUAL_WARNINGS
	#define PRAGMA_ENABLE_OVERLOADED_VIRTUAL_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_OVERLOADED_VIRTUAL_WARNINGS

#ifndef PRAGMA_DISABLE_MISSING_BRACES_WARNINGS
	#define PRAGMA_DISABLE_MISSING_BRACES_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wmissing-braces\"")
#endif // PRAGMA_DISABLE_MISSING_BRACES_WARNINGS

#ifndef PRAGMA_ENABLE_MISSING_BRACES_WARNINGS
	#define PRAGMA_ENABLE_MISSING_BRACES_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_MISSING_BRACES_WARNINGS

#ifndef PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
	#define PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wshadow\"")
#endif // PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS

#ifndef PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS
	#define PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS

#ifndef PRAGMA_DISABLE_CPP20_EXTENSION_WARNINGS
	#define PRAGMA_DISABLE_CPP20_EXTENSION_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wc++20-extensions\"")
#endif // PRAGMA_DISABLE_CPP20_EXTENSION_WARNINGS

#ifndef PRAGMA_ENABLE_CPP20_EXTENSION_WARNINGS
	#define PRAGMA_ENABLE_CPP20_EXTENSION_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_CPP20_EXTENSION_WARNINGS

#ifndef PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS
	#define PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wunreachable-code-aggressive\"")
#endif // PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS

#ifndef PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS
	#define PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS

#ifndef PRAGMA_PUSH_ATTRIBUTE_MINSIZE_FUNCTIONS
	#define PRAGMA_PUSH_ATTRIBUTE_MINSIZE_FUNCTIONS _Pragma("clang attribute push(__attribute__((minsize)), apply_to=function)")
#endif // PRAGMA_PUSH_ATTRIBUTE_MINSIZE_FUNCTIONS

#ifndef PRAGMA_POP_ATTRIBUTE_MINSIZE_FUNCTIONS
	#define PRAGMA_POP_ATTRIBUTE_MINSIZE_FUNCTIONS _Pragma("clang attribute pop")
#endif // PRAGMA_POP_ATTRIBUTE_MINSIZE_FUNCTIONS

#if __has_warning("-Wimplicit-float-conversion")
#define DISABLE_IMPLICIT_FLOAT_CONVERSION_FRAGMENT _Pragma("clang diagnostic ignored \"-Wimplicit-float-conversion\"")
#else
#define DISABLE_IMPLICIT_FLOAT_CONVERSION_FRAGMENT
#endif

#if __has_warning("-Wimplicit-float-conversion")
#define FORCE_IMPLICIT_FLOAT_CONVERSION_FRAGMENT _Pragma("clang diagnostic warning \"-Wimplicit-float-conversion\"")
#else
#define FORCE_IMPLICIT_FLOAT_CONVERSION_FRAGMENT
#endif

#if __has_warning("-Wimplicit-int-conversion")
#define DISABLE_IMPLICIT_INT_CONVERSION_FRAGMENT _Pragma("clang diagnostic ignored \"-Wimplicit-int-conversion\"")
#else
#define DISABLE_IMPLICIT_INT_CONVERSION_FRAGMENT
#endif

#if __has_warning("-Wimplicit-int-conversion")
#define FORCE_IMPLICIT_INT_CONVERSION_FRAGMENT _Pragma("clang diagnostic warning \"-Wimplicit-int-conversion\"")
#else
#define FORCE_IMPLICIT_INT_CONVERSION_FRAGMENT
#endif

#ifndef PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS
	#define PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wfloat-conversion\"") \
		DISABLE_IMPLICIT_FLOAT_CONVERSION_FRAGMENT \
		DISABLE_IMPLICIT_INT_CONVERSION_FRAGMENT \
		_Pragma("clang diagnostic ignored \"-Wc++11-narrowing\"")
#endif // PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

#ifndef PRAGMA_ENABLE_UNSAFE_TYPECAST_WARNINGS
	#define PRAGMA_ENABLE_UNSAFE_TYPECAST_WARNINGS \
		UE_DEPRECATED_MACRO(5.0, "The PRAGMA_ENABLE_UNSAFE_TYPECAST_WARNINGS macro has been deprecated in favor of PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS. To force enable warnings use PRAGMA_FORCE_UNSAFE_TYPECAST_WARNINGS.")
#endif // PRAGMA_ENABLE_UNSAFE_TYPECAST_WARNINGS

#ifndef PRAGMA_FORCE_UNSAFE_TYPECAST_WARNINGS
	#define PRAGMA_FORCE_UNSAFE_TYPECAST_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic warning \"-Wfloat-conversion\"") \
		FORCE_IMPLICIT_FLOAT_CONVERSION_FRAGMENT \
		FORCE_IMPLICIT_INT_CONVERSION_FRAGMENT \
		_Pragma("clang diagnostic warning \"-Wc++11-narrowing\"")
#endif // PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

#ifndef PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
	#define PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS \
	_Pragma("clang diagnostic pop")
#endif // PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS

#ifndef PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS
	#define PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wswitch-enum\"")
#endif // PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

#ifndef PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS
	#define PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS

#if __has_warning("-Wordered-compare-function-pointers")
#define DISABLE_ORDERED_COMPARE_FUNCTION_POINTERS _Pragma("clang diagnostic ignored \"-Wordered-compare-function-pointers\"")
#else
#define DISABLE_ORDERED_COMPARE_FUNCTION_POINTERS
#endif

#ifndef PRAGMA_DISABLE_ORDERED_COMPARE_FUNCTION_POINTERS
	#define PRAGMA_DISABLE_ORDERED_COMPARE_FUNCTION_POINTERS \
		_Pragma("clang diagnostic push") \
		DISABLE_ORDERED_COMPARE_FUNCTION_POINTERS
#endif // PRAGMA_DISABLE_ORDERED_COMPARE_FUNCTION_POINTERS

#ifndef PRAGMA_ENABLE_ORDERED_COMPARE_FUNCTION_POINTERS
	#define PRAGMA_ENABLE_ORDERED_COMPARE_FUNCTION_POINTERS \
	_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_ORDERED_COMPARE_FUNCTION_POINTERS

#ifndef PRAGMA_DISABLE_UNDEFINED_IDENTIFIER_WARNINGS
	#define PRAGMA_DISABLE_UNDEFINED_IDENTIFIER_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wundef\"")
#endif // PRAGMA_DISABLE_UNDEFINED_IDENTIFIER_WARNINGS

#ifndef PRAGMA_ENABLE_UNDEFINED_IDENTIFIER_WARNINGS
	#define PRAGMA_ENABLE_UNDEFINED_IDENTIFIER_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_UNDEFINED_IDENTIFIER_WARNINGS

#ifndef PRAGMA_DISABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS
	#define PRAGMA_DISABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wdelete-non-virtual-dtor\"")
#endif // PRAGMA_DISABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS

#ifndef PRAGMA_ENABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS
	#define PRAGMA_ENABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS

#ifndef PRAGMA_DISABLE_REORDER_WARNINGS
	#define PRAGMA_DISABLE_REORDER_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wreorder\"")
#endif // PRAGMA_DISABLE_REORDER_WARNINGS

#ifndef PRAGMA_ENABLE_REORDER_WARNINGS
	#define PRAGMA_ENABLE_REORDER_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_REORDER_WARNINGS

#ifndef PRAGMA_DISABLE_REGISTER_WARNINGS
	#define PRAGMA_DISABLE_REGISTER_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wregister\"")
#endif // PRAGMA_DISABLE_REGISTER_WARNINGS

#ifndef PRAGMA_ENABLE_REGISTER_WARNINGS
	#define PRAGMA_ENABLE_REGISTER_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_REGISTER_WARNINGS

#ifndef PRAGMA_DISABLE_MACRO_REDEFINED_WARNINGS
	#define PRAGMA_DISABLE_MACRO_REDEFINED_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wmacro-redefined\"")
#endif // PRAGMA_DISABLE_MACRO_REDEFINED_WARNINGS

#ifndef PRAGMA_DISABLE_IGNORED_PRAGMAS_WARNINGS
	#define PRAGMA_DISABLE_IGNORED_PRAGMAS_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wignored-pragmas\"")
#endif // PRAGMA_DISABLE_IGNORED_PRAGMAS_WARNINGS

#ifndef PRAGMA_ENABLE_IGNORED_PRAGMAS_WARNINGS
	#define PRAGMA_ENABLE_IGNORED_PRAGMAS_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_IGNORED_PRAGMAS_WARNINGS

#ifndef PRAGMA_DISABLE_UNUSED_PRIVATE_FIELDS_WARNINGS
#define PRAGMA_DISABLE_UNUSED_PRIVATE_FIELDS_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wunused-private-field\"")
#endif // PRAGMA_DISABLE_UNUSED_PRIVATE_FIELDS_WARNINGS

#ifndef PRAGMA_ENABLE_UNUSED_PRIVATE_FIELDS_WARNINGS
#define PRAGMA_ENABLE_UNUSED_PRIVATE_FIELDS_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_UNUSED_PRIVATE_FIELDS_WARNINGS

#ifndef PRAGMA_ENABLE_MACRO_REDEFINED_WARNINGS
	#define PRAGMA_ENABLE_MACRO_REDEFINED_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_MACRO_REDEFINED_WARNINGS

#if __has_warning("-Wuninitialized-const-reference")
#define DISABLE_UNINITIALIZED_CONST_REFERENCE _Pragma("clang diagnostic ignored \"-Wuninitialized-const-reference\"")
#else
#define DISABLE_UNINITIALIZED_CONST_REFERENCE
#endif

#ifndef PRAGMA_DISABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS
	#define PRAGMA_DISABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS \
		_Pragma("clang diagnostic push") \
        DISABLE_UNINITIALIZED_CONST_REFERENCE
#endif // PRAGMA_DISABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS

#ifndef PRAGMA_ENABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS
	#define PRAGMA_ENABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS

#if __has_warning("-Wenum-constexpr-conversion")
#define DISABLE_ENUM_CONSTEXPR_CONVERSION _Pragma("clang diagnostic ignored \"-Wenum-constexpr-conversion\"")
#else
#define DISABLE_ENUM_CONSTEXPR_CONVERSION
#endif

#ifndef PRAGMA_DISABLE_ENUM_CONSTEXPR_CONVERSION_WARNINGS
#define PRAGMA_DISABLE_ENUM_CONSTEXPR_CONVERSION_WARNINGS \
		_Pragma("clang diagnostic push") \
        DISABLE_ENUM_CONSTEXPR_CONVERSION
#endif // PRAGMA_DISABLE_ENUM_CONSTEXPR_CONVERSION_WARNINGS

#ifndef PRAGMA_ENABLE_ENUM_CONSTEXPR_CONVERSION_WARNINGS
#define PRAGMA_ENABLE_ENUM_CONSTEXPR_CONVERSION_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_ENUM_CONSTEXPR_CONVERSION_WARNINGS

#if __has_warning("-Wdeprecated-copy-with-user-provided-copy")
#define DISABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY _Pragma("clang diagnostic ignored \"-Wdeprecated-copy-with-user-provided-copy\"")
#else
#define DISABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY
#endif

#ifndef PRAGMA_DISABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY_WARNINGS
#define PRAGMA_DISABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY_WARNINGS \
		_Pragma("clang diagnostic push") \
        DISABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY
#endif // PRAGMA_DISABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY_WARNINGS

#ifndef PRAGMA_ENABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY_WARNINGS
#define PRAGMA_ENABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY_WARNINGS

#if __has_warning("-Wnonportable-include-path")
#define DISABLE_NONPORTABLE_INCLUDE_PATH _Pragma("clang diagnostic ignored \"-Wnonportable-include-path\"")
#else
#define DISABLE_NONPORTABLE_INCLUDE_PATH
#endif

#ifndef PRAGMA_DISABLE_NONPORTABLE_INCLUDE_PATH_WARNING
#define PRAGMA_DISABLE_NONPORTABLE_INCLUDE_PATH_WARNING \
		_Pragma("clang diagnostic push") \
        DISABLE_NONPORTABLE_INCLUDE_PATH
#endif // PRAGMA_DISABLE_NONPORTABLE_INCLUDE_PATH_WARNING

#ifndef PRAGMA_ENABLE_NONPORTABLE_INCLUDE_PATH_WARNING
#define PRAGMA_ENABLE_NONPORTABLE_INCLUDE_PATH_WARNING \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_NONPORTABLE_INCLUDE_PATH_WARNING

#ifndef PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	#if __has_warning("-Wcast-function-type-mismatch")
		#define PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS \
			_Pragma("clang diagnostic push") \
			_Pragma("clang diagnostic ignored \"-Wcast-function-type-mismatch\"") // cast from 'FARPROC' (aka 'long long (*)()') to 'blah' (aka '<signature>') converts to incompatible function type
	#else
		#define PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	#endif // __has_warning("-Wcast-function-type-mismatch")
#endif // PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

#ifndef PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	#if __has_warning("-Wcast-function-type-mismatch")
		#define PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS \
			_Pragma("clang diagnostic pop")
	#else
		#define PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	#endif // __has_warning("-Wcast-function-type-mismatch")
#endif // PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

#ifndef PRAGMA_POP
	#define PRAGMA_POP \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_POP

#ifndef PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING
	#define PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING \
		_Pragma("pack(push)") \
		_Pragma("pack()")
#endif // PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING
	
#ifndef PRAGMA_POP_PLATFORM_DEFAULT_PACKING
	#define PRAGMA_POP_PLATFORM_DEFAULT_PACKING \
		_Pragma("pack(pop)")
#endif // PRAGMA_POP_PLATFORM_DEFAULT_PACKING

#ifndef EMIT_CUSTOM_WARNING_AT_LINE
	#define EMIT_CUSTOM_WARNING_AT_LINE(Line, Warning) \
		_Pragma(UE_STRINGIZE(message(Warning)))
#endif // EMIT_CUSTOM_WARNING_AT_LINE

#ifndef EMIT_CUSTOM_ERROR_AT_LINE
	#define EMIT_CUSTOM_ERROR_AT_LINE(Line, Error) \
		_Pragma(UE_STRINGIZE(message(Error)))
#endif // EMIT_CUSTOM_ERROR_AT_LINE

#ifndef UE_COMPILER_THIRD_PARTY_INCLUDES_START
	#define UE_COMPILER_THIRD_PARTY_INCLUDES_START \
		PRAGMA_DISABLE_REORDER_WARNINGS \
		PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS \
		PRAGMA_DISABLE_UNDEFINED_IDENTIFIER_WARNINGS \
		PRAGMA_DISABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS \
		PRAGMA_DISABLE_DEPRECATION_WARNINGS \
		PRAGMA_DISABLE_OVERLOADED_VIRTUAL_WARNINGS \
		PRAGMA_DISABLE_MISSING_BRACES_WARNINGS \
		PRAGMA_DISABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS \
		PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS \
		PRAGMA_DISABLE_ORDERED_COMPARE_FUNCTION_POINTERS \
		PRAGMA_DISABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY_WARNINGS
#endif

#ifndef UE_COMPILER_THIRD_PARTY_INCLUDES_END
	#define UE_COMPILER_THIRD_PARTY_INCLUDES_END \
		PRAGMA_ENABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY_WARNINGS \
		PRAGMA_ENABLE_ORDERED_COMPARE_FUNCTION_POINTERS \
		PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS \
		PRAGMA_ENABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS \
		PRAGMA_ENABLE_MISSING_BRACES_WARNINGS \
		PRAGMA_ENABLE_OVERLOADED_VIRTUAL_WARNINGS \
		PRAGMA_ENABLE_DEPRECATION_WARNINGS \
		PRAGMA_ENABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS \
		PRAGMA_ENABLE_UNDEFINED_IDENTIFIER_WARNINGS \
		PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS \
		PRAGMA_ENABLE_REORDER_WARNINGS
#endif

#endif // __clang__

#include "GenericPlatform/GenericPlatformCompilerPreSetup.h"