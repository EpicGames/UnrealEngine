// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	ClangPlatformMath.h: Clang intrinsic implementations of some platform Math functions
==============================================================================================*/

#pragma once

#ifdef __clang__

#include "Concepts/Integral.h"
#include "GenericPlatform/GenericPlatformMath.h"

// HEADER_UNIT_UNSUPPORTED - Clang not supporting header units

/**
 * Clang implementation of math functions
 **/
struct FClangPlatformMath : public FGenericPlatformMath
{
	/**
	 * Counts the number of leading zeros in the bit representation of the value
	 *
	 * @param Value the value to determine the number of leading zeros for
	 *
	 * @return the number of zeros before the first "on" bit
	 */
	static UE_FORCEINLINE_HINT constexpr uint8 CountLeadingZeros8(uint8 Value)
	{
		return uint8(__builtin_clz((uint32(Value) << 1) | 1) - 23);
	}

	/**
	 * Counts the number of leading zeros in the bit representation of the value
	 *
	 * @param Value the value to determine the number of leading zeros for
	 *
	 * @return the number of zeros before the first "on" bit
	 */
	static UE_FORCEINLINE_HINT constexpr uint32 CountLeadingZeros(uint32 Value)
	{
		return __builtin_clzll((uint64(Value) << 1) | 1) - 31;
	}

	/**
	 * Counts the number of leading zeros in the bit representation of the value
	 *
	 * @param Value the value to determine the number of leading zeros for
	 *
	 * @return the number of zeros before the first "on" bit
	 */
	static UE_FORCEINLINE_HINT constexpr uint64 CountLeadingZeros64(uint64 Value)
	{
		if (Value == 0)
		{
			return 64;
		}

		return __builtin_clzll(Value);
	}

	/**
	 * Counts the number of trailing zeros in the bit representation of the value
	 *
	 * @param Value the value to determine the number of trailing zeros for
	 *
	 * @return the number of zeros after the last "on" bit
	 */
	static UE_FORCEINLINE_HINT constexpr uint32 CountTrailingZeros(uint32 Value)
	{
		if (Value == 0)
		{
			return 32;
		}

		return (uint32)__builtin_ctz(Value);
	}

	/**
	 * Counts the number of trailing zeros in the bit representation of the value
	 *
	 * @param Value the value to determine the number of trailing zeros for
	 *
	 * @return the number of zeros after the last "on" bit
	 */
	static UE_FORCEINLINE_HINT constexpr uint64 CountTrailingZeros64(uint64 Value)
	{
		if (Value == 0)
		{
			return 64;
		}

		return (uint64)__builtin_ctzll(Value);
	}

	static UE_FORCEINLINE_HINT constexpr uint32 FloorLog2(uint32 Value)
	{
		return 31 - __builtin_clz(Value | 1);
	}

	static UE_FORCEINLINE_HINT constexpr uint32 FloorLog2NonZero(uint32 Value)
	{
		return 31 - __builtin_clz(Value);
	}

	static UE_FORCEINLINE_HINT constexpr uint64 FloorLog2_64(uint64 Value)
	{
		return 63 - __builtin_clzll(Value | 1);
	}

	static UE_FORCEINLINE_HINT constexpr uint64 FloorLog2NonZero_64(uint64 Value)
	{
		return 63 - __builtin_clzll(Value);
	}

	static UE_FORCEINLINE_HINT constexpr uint32 CeilLogTwo(uint32 Arg)
	{
		// if Arg is 0, change it to 1 so that we return 0
		Arg = Arg ? Arg : 1;
		return 32 - CountLeadingZeros(Arg - 1);
	}

	static UE_FORCEINLINE_HINT constexpr uint64 CeilLogTwo64(uint64 Arg)
	{
		// if Arg is 0, change it to 1 so that we return 0
		Arg = Arg ? Arg : 1;
		return 64 - CountLeadingZeros64(Arg - 1);
	}

	static UE_FORCEINLINE_HINT constexpr uint32 RoundUpToPowerOfTwo(uint32 Arg)
	{
		return uint32(1) << CeilLogTwo(Arg);
	}

	static UE_FORCEINLINE_HINT constexpr uint64 RoundUpToPowerOfTwo64(uint64 Arg)
	{
		return uint64(1) << CeilLogTwo64(Arg);
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT constexpr int32 CountBits(uint32 Bits)
	{
		return __builtin_popcount(Bits);
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT constexpr int32 CountBits64(uint64 Bits)
	{
		return __builtin_popcountll(Bits);
	}

	template <UE::CIntegral T>
		requires (sizeof(T) > 4)
	[[nodiscard]] UE_REWRITE static constexpr int32 CountBits(T Bits)
	{
		UE_STATIC_DEPRECATE(5.8, sizeof(T) > 4, "CountBits for 64-bit values has been deprecated, please use CountBits64 instead.");
		return CountBits64(Bits);
	}

	/**
	 * Adds two integers of any integer type, checking for overflow.
	 * If there was overflow, it returns false, and OutResult may or may not be written.
	 * If there wasn't overflow, it returns true, and the result of the addition is written to OutResult.
	 */
	template <UE::CIntegral IntType>
	static UE_FORCEINLINE_HINT bool AddAndCheckForOverflow(IntType A, IntType B, IntType& OutResult)
	{
		return !__builtin_add_overflow(A, B, &OutResult);
	}

	/**
	 * Subtracts two integers of any integer type, checking for overflow.
	 * If there was overflow, it returns false, and OutResult may or may not be written.
	 * If there wasn't overflow, it returns true, and the result of the subtraction is written to OutResult.
	 */
	template <UE::CIntegral IntType>
	static UE_FORCEINLINE_HINT bool SubtractAndCheckForOverflow(IntType A, IntType B, IntType& OutResult)
	{
		return !__builtin_sub_overflow(A, B, &OutResult);
	}

	/**
	 * Multiplies two integers of any integer type, checking for overflow.
	 * If there was overflow, it returns false, and OutResult may or may not be written.
	 * If there wasn't overflow, it returns true, and the result of the multiplication is written to OutResult.
	 */
	template <UE::CIntegral IntType>
	static UE_FORCEINLINE_HINT bool MultiplyAndCheckForOverflow(IntType A, IntType B, IntType& OutResult)
	{
		return !__builtin_mul_overflow(A, B, &OutResult);
	}
};

#endif //~__clang__