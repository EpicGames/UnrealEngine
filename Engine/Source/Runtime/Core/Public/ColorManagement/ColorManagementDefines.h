// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE { namespace Color {

// TODO: ENCODING_TYPES_VER and COLORSPACE_VER are the only per-enum schema
// versions embedded in texture DDC keys (see TextureDerivedData.cpp and
// TextureDerivedDataBuildUtils.cpp). No other texture-facing enum carries
// this pattern; the broader DDC key derivation covers schema changes for
// everything else. These should be removed in favor of standard DDC
// versioning, and breaking enum changes handled through the usual pipeline.

/** Increment upon breaking changes to the EEncoding enum (renames, renumbering, or semantic changes to existing values). Pure additions do not require a bump. Note that changing this forces a rebuild of textures that rely on it. */
inline constexpr uint32 ENCODING_TYPES_VER = 4;

/** Increment upon breaking changes to the EColorSpace and EChromaticAdaptationMethod enums (renames, renumbering, or semantic changes to existing values). Pure additions do not require a bump. Note that changing this forces a rebuild of textures that rely on it. */
inline constexpr uint32 COLORSPACE_VER = 2;

/** List of available encodings/transfer functions.
*
* NOTE: This list is replicated as a UENUM in TextureDefines.h, and both should always match.
*/
enum class EEncoding : uint8
{
	None        = 0,
	Linear      = 1,
	sRGB        = 2,
	ST2084      = 3,
	Gamma22     = 4,
	BT1886      = 5,
	Gamma26     = 6,
	Cineon      = 7,
	REDLog      = 8,
	REDLog3G10  = 9,
	SLog1       = 10,
	SLog2       = 11,
	SLog3       = 12,
	AlexaV3LogC = 13,
	CanonLog    = 14,
	ProTune     = 15,
	VLog        = 16,
	HLG         = 17,
	Max         = 18,
};

/**
 * List of standard reference-white conventions that can be applied as a linear-side
 * normalization alongside an EEncoding. Applied so that diffuse white lands at UE
 * scene-linear 1.0 for the chosen standard. Resolve to a numeric scale with
 * UE::Color::GetReferenceWhiteLinearScale(EEncoding, EReferenceWhite).
 *
 * NOTE: This list is mirrored as a UENUM in BaseMediaSourceColorSettings.h, and both
 * should always match.
 */
enum class EReferenceWhite : uint8
{
	/** The source reference white is not overridden, defer to system/default behavior. */
	None                 = 0,
	/** Explicitly preserve input/native scaling. */
	DisableNormalization = 1,
	/** ITU-R BT.2408 Reference White. 203 cd/m2 for PQ (ST2084); 75% signal (~3.77x BT.2100 linear) for HLG. */
	BT2408               = 2,
	/** ITU-R BT.1886 SDR reference white (100 cd/m2). Applies to PQ (100-nit paper white) and HLG (in-house 100-nit convention on a 1000 cd/m2 BT.2100 reference display). */
	BT1886               = 3,
	Max                  = 4,
};

/** List of available color spaces. (Increment COLORSPACE_VER upon breaking changes to the list.)
* 
* NOTE: This list is partially replicated as a UENUM in TextureDefines.h: any type exposed to textures should match the enum value below.
*/
enum class EColorSpace : uint8
{
	None = 0,
	sRGB = 1,
	Rec2020 = 2,
	ACESAP0 = 3,
	ACESAP1 = 4,
	P3DCI = 5,
	P3D65 = 6,
	REDWideGamut = 7,
	SonySGamut3 = 8,
	SonySGamut3Cine = 9,
	AlexaWideGamut = 10,
	CanonCinemaGamut = 11,
	GoProProtuneNative = 12,
	PanasonicVGamut = 13,
	PLASA_E1_54 = 14,
	ACESCAM16 = 15,
	Max,
};


/** List of available chromatic adaptation methods.
* 
* NOTE: This list is replicated as a UENUM in TextureDefines.h, and both should always match.
*/
enum class EChromaticAdaptationMethod : uint8
{
	None = 0,
	Bradford = 1,
	CAT02 = 2,
	Max,
};

/** Default method used across the engine for chromatic adaptation. */
inline constexpr EChromaticAdaptationMethod DEFAULT_CHROMATIC_ADAPTATION_METHOD = EChromaticAdaptationMethod::Bradford;

/** List of standard white points. */
enum class EWhitePoint : uint8
{
	CIE1931_D65 = 0,
	ACES_D60 = 1,
	DCI_CalibrationWhite,
	Max,
};

} } // end namespace UE::Color
