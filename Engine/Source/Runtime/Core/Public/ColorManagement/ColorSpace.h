// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ColorManagement/ColorManagementDefines.h"
#include "Containers/StaticArray.h"
#include "CoreTypes.h"
#include "Math/Color.h"
#include "Math/Matrix.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Templates/UnrealTypeTraits.h"

namespace UE { namespace Color {

/**
 * Convert chromaticity coordinate and luminance to CIE XYZ tristimulus values.
 *
 * @return FVector3d
 */
inline FVector3d xyYToXYZ(const FVector3d& xyY)
{
	double Divisor = FMath::Max(xyY[1], 1e-10);

	return FVector3d(
		xyY[0] * xyY[2] / Divisor,
		xyY[2],
		(1.0 - xyY[0] - xyY[1]) * xyY[2] / Divisor
	);
}

/**
 * Convert CIE XYZ tristimulus values to chromaticitiy coordinates and luminance.
 *
 * @return FVector3d
 */
inline FVector3d XYZToxyY(const FVector3d& XYZ)
{
	double Divisor = XYZ[0] + XYZ[1] + XYZ[2];
	if (Divisor == 0.0)
	{
		Divisor = 1e-10;
	}

	return FVector3d(
		XYZ[0] / Divisor,
		XYZ[1] / Divisor,
		XYZ[1]
	);
}

/**
 * Convenience function to get the transposed matrix, i.e. for pre-multiplied shader matrices.
 *
 * @return TMatrix<T>
 */
template<typename T>
inline UE::Math::TMatrix<T> Transpose(const FMatrix44d& Transform)
{
	if constexpr (std::is_same_v<T, double>)
	{
		return Transform.GetTransposed();
	}
	else
	{
		return UE::Math::TMatrix<T>(Transform).GetTransposed();
	}
}

/**
 * Get standard white point coordinates.
 *
 * @param InWhitePoint White point type.
 * @return FVector2d Chromaticity coordinates.
 */
inline FVector2d GetWhitePoint(EWhitePoint InWhitePoint)
{
	switch (InWhitePoint)
	{
	case UE::Color::EWhitePoint::CIE1931_D65:
		return FVector2d(0.3127, 0.3290);
	case UE::Color::EWhitePoint::ACES_D60:
		return FVector2d(0.32168, 0.33767);
	case UE::Color::EWhitePoint::DCI_CalibrationWhite:
		return FVector2d(0.314, 0.351);

	default:
		checkNoEntry();
		return FVector2d();
	}
}


/** Color space definition as 4 chromaticity coordinates, in double precision internally. */
class FColorSpace
{
public:
	/**
	 * Get the global engine working color space (as a singleton).
	 * 
	 * @return FColorSpace working color space
	 */
	static CORE_API const FColorSpace& GetWorking();

	/**
	 * Set the global engine working color space (as a singleton).
	 * 
	 * @param ColorSpace working color space
	 */
	static CORE_API void SetWorking(FColorSpace ColorSpace);

	/**
	 * Get a (statically cached) sRGB color space.
	 *
	 * @return FColorSpace sRGB color space
	 */
	static CORE_API const FColorSpace& GetSRGB();

	/**
	 * Get a (statically cached) Rec2020 color space.
	 *
	 * @return FColorSpace Rec2020 color space
	 */
	static CORE_API const FColorSpace& GetRec2020();

	/**
	 * Get a (statically cached) P3-D65 color space.
	 *
	 * @return FColorSpace P3-D65 color space
	 */
	static CORE_API const FColorSpace& GetP3D65();

	/** Constructor */
	FColorSpace() {}

	/**
	* Constructor
	*
	* @param InRed Chromaticity 2D coordinates for the red color.
	* @param InGreen Chromaticity 2D coordinates for the green color.
	* @param InBlue Chromaticity 2D coordinates for the blue color.
	* @param InWhite Chromaticity 2D coordinates for the white point.
	*/
	CORE_API explicit FColorSpace(const FVector2d& InRed, const FVector2d& InGreen, const FVector2d& InBlue, const FVector2d& InWhite);

	/**
	* Constructor
	*
	* @param ColorSpaceType Color space type.
	*/
	CORE_API explicit FColorSpace(UE::Color::EColorSpace ColorSpaceType);

	FColorSpace(FColorSpace&&) = default;
	FColorSpace(const FColorSpace&) = default;
	FColorSpace& operator=(FColorSpace&&) = default;
	FColorSpace& operator=(const FColorSpace&) = default;

	/**
	* Make the chromaticities of the color space type.
	* 
	* @return TStaticArray of four chromaticities
	*/
	CORE_API static TStaticArray<FVector2d, 4> MakeChromaticities(UE::Color::EColorSpace ColorSpaceType);

	/**
	* Getter for the color space chromaticity coordinates.
	* 
	* @param OutRed FVector2d for the red color chromaticity coordinate.
	* @param OutGreen FVector2d for the green color chromaticity coordinate.
	* @param OutBlue FVector2d for the blue color chromaticity coordinate.
	* @param OutWhite FVector2d for the white color chromaticity coordinate.
	*/
	template<typename T>
	inline void GetChromaticities(UE::Math::TVector2<T>& OutRed, UE::Math::TVector2<T>& OutGreen, UE::Math::TVector2<T>& OutBlue, UE::Math::TVector2<T>& OutWhite) const
	{
		OutRed		= Chromaticities[0];
		OutGreen	= Chromaticities[1];
		OutBlue		= Chromaticities[2];
		OutWhite	= Chromaticities[3];
	}

	/**
	* Gets the color space's red chromaticity coordinates.
	*
	* @return FVector2d xy coordinates.
	*/
	inline const FVector2d& GetRedChromaticity() const
	{
		return Chromaticities[0];
	}

	/**
	* Gets the color space's green chromaticity coordinates.
	*
	* @return FVector2d xy coordinates.
	*/
	inline const FVector2d& GetGreenChromaticity() const
	{
		return Chromaticities[1];
	}

	/**
	* Gets the color space's blue chromaticity coordinates.
	*
	* @return FVector2d xy coordinates.
	*/
	inline const FVector2d& GetBlueChromaticity() const
	{
		return Chromaticities[2];
	}

	/**
	* Gets the color space's white point chromaticity coordinates.
	*
	* @return FVector2d xy coordinates.
	*/
	inline const FVector2d& GetWhiteChromaticity() const
	{
		return Chromaticities[3];
	}

	/**
	* Gets the RGB-to-XYZ conversion matrix.
	*
	* @return FMatrix conversion matrix.
	*/
	inline const FMatrix44d& GetRgbToXYZ() const
	{
		return RgbToXYZ;
	}

	/**
	* Gets the XYZ-to-RGB conversion matrix.
	*
	* @return FMatrix conversion matrix.
	*/
	inline const FMatrix44d& GetXYZToRgb() const
	{
		return XYZToRgb;
	}

	/**
	 * Check against another color space for equality.
	 *
	 * @param ColorSpace The vector to check against.
	 * @return true if the color spaces are equal, false otherwise.
	 */
	inline bool operator==(const FColorSpace& ColorSpace) const
	{
		return Chromaticities == ColorSpace.Chromaticities;
	}

	/**
	 * Check against another color space for inequality.
	 *
	 * @param ColorSpace The color space to check against.
	 * @return true if the color spaces are not equal, false otherwise.
	 */
	inline bool operator!=(const FColorSpace& ColorSpace) const
	{
		return Chromaticities != ColorSpace.Chromaticities;
	}

	/**
	 * Check against another color space for equality, within specified error limits.
	 *
	 * @param ColorSpace The color space to check against.
	 * @param Tolerance Error tolerance.
	 * @return true if the color spaces are equal within tolerance limits, false otherwise.
	 */
	CORE_API bool Equals(const FColorSpace& ColorSpace, double Tolerance = 1.e-7) const;


	/**
	 * Check against color space chromaticities for equality, within specified error limits.
	 *
	 * @param InChromaticities The color space chromaticities to check against.
	 * @param Tolerance Error tolerance.
	 * @return true if the color spaces are equal within tolerance limits, false otherwise.
	 */
	CORE_API bool Equals(const TStaticArray<FVector2d, 4>& InChromaticities, double Tolerance = 1.e-7) const;

	/**
	 * Convenience function to verify if the color space matches the engine's default sRGB chromaticities.
	 *
	 * @return true if sRGB.
	 */
	CORE_API bool IsSRGB() const;

	/**
	* Converts temperature in Kelvins of a black body radiator to an RGB color in the current space.
	*/
	CORE_API FLinearColor MakeFromColorTemperature(float Temp) const;

	/**
	 * Calculate the color's luminance value in the current space.
	 *
	 * @param Color The sampled color.
	 * @return float Luminance value.
	 */
	CORE_API float GetLuminance(const FLinearColor& Color) const;

	/**
	 * Get the luminance factors in the current space.
	 *
	 * @return FLinearColor Luminance factors.
	 */
	CORE_API FLinearColor GetLuminanceFactors() const;

private:

	CORE_API FMatrix44d CalcRgbToXYZ() const;

	/** Red, green, blue, white chromaticities, in order. */
	TStaticArray<FVector2d, 4> Chromaticities;

	FMatrix44d RgbToXYZ;
	FMatrix44d XYZToRgb;

	bool bIsSRGB;

public:

	/**
	 * Serializer.
	 *
	 * @param Ar The Serialization Archive.
	 * @param CS The Color Space being serialized.
	 * @return Reference to the Archive after serialization.
	 */
	friend FArchive& operator<<(FArchive& Ar, FColorSpace& CS)
	{
		return Ar << CS.Chromaticities;
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}
};


struct FColorSpaceTransform : FMatrix44d
{
	/**
	* Constructor: create a color space transformation matrix from a source to a target color space using the RGB-XYZ-RGB conversions.
	*
	* @param Source Source color space.
	* @param Target Target color space.
	* @param Method Chromatic adapation method.
	*/
	CORE_API explicit FColorSpaceTransform(const FColorSpace& Src, const FColorSpace& Dst, EChromaticAdaptationMethod Method = DEFAULT_CHROMATIC_ADAPTATION_METHOD);

	/**
	* Constructor: create a color space transformation matrix from a raw matrix.
	*
	* @param Matrix Color space transformation matrix.
	*/
	CORE_API explicit FColorSpaceTransform(FMatrix44d Matrix);

	/**
	* Apply color space transform to FLinearColor.
	* 
	* @param Color Color to transform.
	*/
	CORE_API FLinearColor Apply(const FLinearColor& Color) const;

	/**
	* Calculate the chromatic adaptation matrix using the specific method.
	*
	* @param SourceXYZ Source color in XYZ space.
	* @param TargetXYZ Target color in XYZ space.
	* @param Method Adaptation method (None, Bradford, CAT02).
	*/
	CORE_API static FMatrix44d CalcChromaticAdaptionMatrix(FVector3d SourceXYZ, FVector3d TargetXYZ, EChromaticAdaptationMethod Method = DEFAULT_CHROMATIC_ADAPTATION_METHOD);

	/**
	* Convenience function to get a (statically cached) conversion from sRGB/Rec709 to the working color space.
	*/
	CORE_API static const FColorSpaceTransform& GetSRGBToWorkingColorSpace();

	/**
	 * Convenience function to get a (statically cached) conversion from the working color space to sRGB/Rec709.
	*/
	CORE_API static const FColorSpaceTransform& GetWorkingColorSpaceToSRGB();
};

} }  // end namespace UE::Color
