// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ColorManagement/ColorManagementDefines.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "Math/Color.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/Function.h"

namespace UE { namespace Color {

inline float Linear(float Value)
{
	return Value;
}

/**
* Encode value to sRGB.
*
* @return float encoded value.
*/
inline float EncodeSRGB(float Value)
{
	if (Value <= 0.04045f / 12.92f)
	{
		return Value * 12.92f;
	}
	else
	{
		return  FGenericPlatformMath::Pow(Value, (1.0f / 2.4f)) * 1.055f - 0.055f;
	}
}

/**
* Decode value with an sRGB encoding.
*
* @return float decoded value.
*/
inline float DecodeSRGB(float Value)
{
	if (Value <= 0.04045f)
	{
		return Value / 12.92f;
	}
	else
	{
		return  FGenericPlatformMath::Pow((Value + 0.055f) / 1.055f, 2.4f);
	}
}

/**
 * Encode normalized luminance to SMPTE ST 2084:2014.
 *
 * @return float encoded value.
 */
inline float EncodeNormalizedToST2084(float ZeroToOne)
{
	constexpr float m1 = 2610 / 4096.0f * (1.0f / 4.0f);
	constexpr float m2 = 2523 / 4096.0f * 128.0f;
	constexpr float c1 = 3424 / 4096.0f;
	constexpr float c2 = 2413 / 4096.0f * 32.f;
	constexpr float c3 = 2392 / 4096.0f * 32.f;

	float Value = FGenericPlatformMath::Pow(ZeroToOne, m1);
	return FGenericPlatformMath::Pow((c1 + c2 * Value) / (c3 * Value + 1), m2);
}

/**
 * Encode nits to SMPTE ST 2084:2014.
 *
 * @return float encoded value.
 */
inline float EncodeST2084(float Nits)
{
	constexpr float Lp = 10000.f;
	return EncodeNormalizedToST2084(Nits / Lp);
}

/**
 * Decode value with a SMPTE ST 2084:2014 encoding.
 *
 * @return float decoded normalized luminance.
 */
inline float DecodeNormalizedFromST2084(float Value)
{
	constexpr float m1 = 2610 / 4096.0f * (1.0f / 4.0f);
	constexpr float m2 = 2523 / 4096.0f * 128.0f;
	constexpr float c1 = 3424 / 4096.0f;
	constexpr float c2 = 2413 / 4096.0f * 32.f;
	constexpr float c3 = 2392 / 4096.0f * 32.f;

	const float Vp = FGenericPlatformMath::Pow(Value, 1.0f / m2);
	Value = FGenericPlatformMath::Max(0.0f, Vp - c1);
	return FGenericPlatformMath::Pow((Value / (c2 - c3 * Vp)), 1.0f / m1);
}

/**
 * Decode value with a SMPTE ST 2084:2014 encoding.
 *
 * @return float decoded nits.
 */
inline float DecodeST2084(float Value)
{
	constexpr float Lp = 10000.f;
	return DecodeNormalizedFromST2084(Value) * Lp;
}

/**
* Encode value to Gamma 2.2.
*
* @return float encoded value.
*/
inline float EncodeGamma22(float Value)
{
	return FGenericPlatformMath::Pow(Value, 1.0f / 2.2f);
}


/**
* Decode value with a Gamma 2.2 encoding.
*
* @return float decoded value.
*/
inline float DecodeGamma22(float Value)
{
	return FGenericPlatformMath::Pow(Value, 2.2f);
}

/**
* Encode value to ITU-R BT.1886.
*
* @return float encoded value.
*/
inline float EncodeBT1886(float Value)
{
	constexpr float L_B = 0;
	constexpr float L_W = 1;
	constexpr float Gamma = 2.40f;
	constexpr float GammaInv = 1.0f / Gamma;
	float N = FGenericPlatformMath::Pow(L_W, GammaInv) - FGenericPlatformMath::Pow(L_B, GammaInv);
	float A = FGenericPlatformMath::Pow(N, Gamma);
	float B = FGenericPlatformMath::Pow(L_B, GammaInv) / N;
	return FGenericPlatformMath::Pow(Value / A, GammaInv) - B;
}

/**
* Encode value to Gamma 2.6.
*
* @return float encoded value.
*/
inline float EncodeGamma26(float Value)
{
	return FGenericPlatformMath::Pow(Value, 1.0f / 2.6f);
}


/**
* Decode value with a Gamma 2.6 encoding.
*
* @return float decoded value.
*/
inline float DecodeGamma26(float Value)
{
	return FGenericPlatformMath::Pow(Value, 2.6f);
}

/**
* Decode value with an ITU-R BT.1886 encoding.
*
* @return float decoded value.
*/
inline float DecodeBT1886(float Value)
{
	constexpr float L_B = 0;
	constexpr float L_W = 1;
	constexpr float Gamma = 2.40f;
	constexpr float GammaInv = 1.0f / Gamma;
	float N = FGenericPlatformMath::Pow(L_W, GammaInv) - FGenericPlatformMath::Pow(L_B, GammaInv);
	float A = FGenericPlatformMath::Pow(N, Gamma);
	float B = FGenericPlatformMath::Pow(L_B, GammaInv) / N;
	return A * FGenericPlatformMath::Pow(FGenericPlatformMath::Max(Value + B, 0.0f), Gamma);
}

/**
* Encode value to Cineon.
*
* @return float encoded value.
*/
inline float EncodeCineon(float Value)
{
	const float BlackOffset = FGenericPlatformMath::Pow(10.0f, (95.0f - 685.0f) / 300.0f);
	return (685.0f + 300.0f * FGenericPlatformMath::LogX(10.0f, Value * (1.0f - BlackOffset) + BlackOffset)) / 1023.0f;
}

/**
* Decode value with a Cineon encoding.
*
* @return float decoded value.
*/
inline float DecodeCineon(float Value)
{
	const float BlackOffset = FGenericPlatformMath::Pow(10.0f, (95.0f - 685.0f) / 300.0f);
	return (FGenericPlatformMath::Pow(10.0f, (1023.0f * Value - 685.0f) / 300.0f) - BlackOffset) / (1.0f - BlackOffset);
}

/**
* Encode value to RED Log.
*
* @return float encoded value.
*/
inline float EncodeREDLog(float Value)
{
	const float BlackOffset = FGenericPlatformMath::Pow(10.0f, (0.0f - 1023.0f) / 511.0f);
	return (1023.0f + 511.0f * FGenericPlatformMath::LogX(10.0f, Value * (1.0f - BlackOffset) + BlackOffset)) / 1023.0f;
}

/**
* Decode value with a RED Log encoding.
*
* @return float decoded value.
*/
inline float DecodeREDLog(float Value)
{
	const float BlackOffset = FGenericPlatformMath::Pow(10.0f, (0.0f - 1023.0f) / 511.0f);
	return (FGenericPlatformMath::Pow(10.0f, (1023.0f * Value - 1023.0f) / 511.0f) - BlackOffset) / (1.0f - BlackOffset);
}

/**
* Encode value to RED Log3G10.
*
* @return float encoded value.
*/
inline float EncodeREDLog3G10(float Value)
{
	constexpr float A = 0.224282f;
	constexpr float B = 155.975327f;
	constexpr float C = 0.01f;
	constexpr float G = 15.1927f;

	Value += C;

	if (Value < 0.0f)
	{
		return Value * G;
	}
	else
	{
		return FGenericPlatformMath::Sign(Value) * A * FGenericPlatformMath::LogX(10.0f, B * FGenericPlatformMath::Abs(Value) + 1.0f);
	}
}

/**
* Decode value with a RED Log3G10 encoding.
*
* @return float decoded value.
*/
inline float DecodeREDLog3G10(float Value)
{
	constexpr float A = 0.224282f;
	constexpr float B = 155.975327f;
	constexpr float C = 0.01f;
	constexpr float G = 15.1927f;

	if (Value < 0.0f)
	{
		Value /= G;
	}
	else
	{
		Value = FGenericPlatformMath::Sign(Value) * (FGenericPlatformMath::Pow(10.0f, FGenericPlatformMath::Abs(Value) / A ) - 1.0f) / B;
	}

	return Value - C;
}

/**
* Encode value to Sony S-Log1.
*
* @return float encoded value.
*/
inline float EncodeSLog1(float Value)
{
	Value /= 0.9f;
	Value = 0.432699f * FGenericPlatformMath::LogX(10.0f, Value + 0.037584f) + 0.616596f + 0.03f;
	return (Value * 219.0f + 16.0f) * 4.0f / 1023.0f;
}

/**
* Decode value with a Sony S-Log1 encoding.
*
* @return float decoded value.
*/
inline float DecodeSLog1(float Value)
{
	Value = ((Value * 1023.f) / 4.0f - 16.0f) / 219.0f;
	Value = FGenericPlatformMath::Pow(10.0f, (Value - 0.616596f - 0.03f) / 0.432699f) - 0.037584f;
	return Value * 0.9f;
}

/**
* Encode value to Sony S-Log2.
*
* @return float encoded value.
*/
inline float EncodeSLog2(float Value)
{
	if (Value >= 0.0f)
	{
		return (64.0f + 876.0f * (0.432699f * FGenericPlatformMath::LogX(10.0f, 155.0f * Value / 197.1f + 0.037584f) + 0.646596f)) / 1023.f;
	}
	else
	{
		return (64.0f +876.0f * (Value * 3.53881278538813f / 0.9f) + 0.646596f + 0.030001222851889303f) / 1023.f;
	}
}

/**
* Decode value with a Sony S-Log2 encoding.
*
* @return float decoded value.
*/
inline float DecodeSLog2(float Value)
{
	if (Value >= (64.f + 0.030001222851889303f * 876.f) / 1023.f)
	{
		return 197.1f * (FGenericPlatformMath::Pow(10.0f, ((Value * 1023.f - 64.f) / 876.f - 0.646596f) / 0.432699f) - 0.037584f) / 155.f;
	}
	else
	{
		return 0.9f * ((Value * 1023.f - 64.f) / 876.f - 0.030001222851889303f) / 3.53881278538813f;
	}
}

/**
* Encode value to Sony S-Log3.
*
* @return float encoded value.
*/
inline float EncodeSLog3(float Value)
{
	if (Value >= 0.01125000f)
	{
		return (420.0f + FGenericPlatformMath::LogX(10.0f, (Value + 0.01f) / 0.19f) * 261.5f) / 1023.0f;
	}
	else
	{
		return (Value * 76.2102946929f / 0.01125f + 95.0f) / 1023.0f;
	}
}

/**
* Decode value with a Sony S-Log3 encoding.
*
* @return float decoded value.
*/
inline float DecodeSLog3(float Value)
{
	if (Value >= 171.2102946929f / 1023.0f)
	{
		return (FGenericPlatformMath::Pow(10.0f, (Value * 1023.0f - 420.f) / 261.5f)) * 0.19f - 0.01f;
	}
	else
	{
		return (Value * 1023.0f - 95.0f) * 0.01125000f / (171.2102946929f - 95.0f);
	}
}

/**
* Encode value to ARRI Alexa LogC.
*
* @return float encoded value.
*/
inline float EncodeArriAlexaV3LogC(float Value)
{
	constexpr float cut = 0.010591f;
	constexpr float a = 5.555556f;
	constexpr float b = 0.052272f;
	constexpr float c = 0.247190f;
	constexpr float d = 0.385537f;
	constexpr float e = 5.367655f;
	constexpr float f = 0.092809f;

	if (Value > cut)
	{
		return c * FGenericPlatformMath::LogX(10.0f, a * Value + b) + d;
	}
	else
	{
		return e * Value + f;
	}
}

/**
* Decode value with an ARRI Alexa LogC encoding.
*
* @return float decoded value.
*/
inline float DecodeArriAlexaV3LogC(float Value)
{
	constexpr float cut = 0.010591f;
	constexpr float a = 5.555556f;
	constexpr float b = 0.052272f;
	constexpr float c = 0.247190f;
	constexpr float d = 0.385537f;
	constexpr float e = 5.367655f;
	constexpr float f = 0.092809f;
	
	if (Value > e * cut + f)
	{
		return (FGenericPlatformMath::Pow(10.0f, (Value - d) / c) - b) / a;
	}
	else
	{
		return (Value - f) / e;
	}
}

/**
* Encode value to Canon Log.
*
* @return float encoded value.
*/
inline float EncodeCanonLog(float Value)
{
	if (Value < 0.0f)
	{
		return -(0.529136f * (FGenericPlatformMath::LogX(10.0f, -Value * 10.1596f + 1.0f)) - 0.0730597f);
	}
	else
	{
		return 0.529136f * FGenericPlatformMath::LogX(10.0f, 10.1596f * Value + 1.0f) + 0.0730597f;
	}
}


/**
* Decode value with a Canon Log encoding.
*
* @return float decoded value.
*/
inline float DecodeCanonLog(float Value)
{
	if (Value < 0.0730597f)
	{
		return -(FGenericPlatformMath::Pow(10.0f, (0.0730597f - Value) / 0.529136f) - 1.0f) / 10.1596f;
	}
	else
	{
		return (FGenericPlatformMath::Pow(10.0f, (Value - 0.0730597f) / 0.529136f) - 1.0f) / 10.1596f;
	}
}

/**
* Encode value to GoPro ProTune.
*
* @return float encoded value.
*/
inline float EncodeGoProProTune(float Value)
{
	return FGenericPlatformMath::Loge(Value * 112.f + 1.0f) / FGenericPlatformMath::Loge(113.0f);
}

/**
* Decode value with a GoPro ProTune encoding.
*
* @return float decoded value.
*/
inline float DecodeGoProProTune(float Value)
{
	return (FGenericPlatformMath::Pow(113.f, Value) - 1.0f) / 112.f;
}

/**
* Encode value to Panasonic V-Log.
*
* @return float encoded value.
*/
inline float EncodePanasonicVLog(float Value)
{
	constexpr float b = 0.00873f;
	constexpr float c = 0.241514f;
	constexpr float d = 0.598206f;

	if( Value < 0.01f)
	{
		return 5.6f * Value + 0.125f;
	}
	else
	{
		return c * FGenericPlatformMath::LogX(10.0f, Value + b) + d;
	}
}

/**
* Decode value with a Panasonic V-Log encoding.
*
* @return float decoded value.
*/
inline float DecodePanasonicVLog(float Value)
{
	constexpr float b = 0.00873f;
	constexpr float c = 0.241514f;
	constexpr float d = 0.598206f;

	if (Value < 0.181f)
	{
		return (Value - 0.125f) / 5.6f;
	}
	else
	{
		return FGenericPlatformMath::Pow(10.0f, (Value - d) / c) - b;
	}
}

/** HLG (Hybrid Log-Gamma) constants per ITU-R BT.2100-3. */
namespace HLG
{
	constexpr float a = 0.17883277f;
	constexpr float b = 1.0f - 4.0f * a;
	constexpr float c = 0.559910729529562f; // 0.5 - a * ln(4a)
}

/**
 * HLG reference OETF (Opto-Electronic Transfer Function) per ITU-R BT.2100-3.
 * Maps a scene-referred linear value E in [0, 1] (BT.2100 literal) to a non-linear
 * signal E' in [0, 1]. Negative values are supported via sign mirroring.
 *
 * BT.2100 is scene-referred on this side of the transfer. For UE scene-linear
 * semantics (diffuse white = 1.0 rather than BT.2100 scene peak = 1.0), callers
 * should pair this with EReferenceWhite::BT2408 and GetReferenceWhiteLinearScale
 * to convert between conventions.
 *
 * @param Value Scene-referred linear value (BT.2100 literal, [0, 1]).
 * @return Non-linear HLG signal.
 */
inline float EncodeHLG(float Value)
{
	const float E = FMath::Abs(Value);
	float Out = 0.0f;

	if (E <= 1.0f / 12.0f)
	{
		Out = FMath::Sqrt(3.0f * E);
	}
	else
	{
		Out = HLG::a * FMath::Loge(12.0f * E - HLG::b) + HLG::c;
	}

	return FMath::CopySign(Out, Value);
}

/**
 * HLG reference inverse OETF per ITU-R BT.2100-3.
 * Maps a non-linear HLG signal E' in [0, 1] back to a scene-referred linear value
 * E in [0, 1] (BT.2100 literal). Negative values are supported via sign mirroring.
 *
 * BT.2100 is scene-referred on this side of the transfer. For UE scene-linear
 * semantics (diffuse white = 1.0 rather than BT.2100 scene peak = 1.0), callers
 * should pair this with EReferenceWhite::BT2408 and GetReferenceWhiteLinearScale
 * to convert between conventions.
 *
 * @param Value Non-linear HLG signal.
 * @return Scene-referred linear value (BT.2100 literal, [0, 1]).
 */
inline float DecodeHLG(float Value)
{
	const float Eprime = FMath::Abs(Value);
	float Out = 0.0f;

	if (Eprime <= 0.5f)
	{
		Out = Eprime * Eprime / 3.0f;
	}
	else
	{
		Out = FMath::Exp((Eprime - HLG::c) / HLG::a) + HLG::b;
		Out /= 12.0f;
	}

	return FMath::CopySign(Out, Value);
}

/**
 * Full BT.2100-3 HLG encoding: inverse OOTF followed by OETF, with optional black lift.
 * Converts display-referred linear light (cd/m2) to a non-linear HLG signal.
 *
 * @param LinearDisplayReferred Display-referred linear color in cd/m2.
 * @param PeakLuminance Display peak luminance in cd/m2 (default 1000).
 * @param BlackLuminance Display black luminance in cd/m2 (default 0).
 * @return Encoded HLG signal.
 */
CORE_API FLinearColor EncodeHLG_2100(const FLinearColor& LinearDisplayReferred, const float PeakLuminance = 1000.0f, const float BlackLuminance = 0.0f);

/**
 * Full BT.2100-3 HLG decoding (EOTF): black lift, inverse OETF, then OOTF.
 * Converts a non-linear HLG signal to display-referred linear light (cd/m2).
 * The shader equivalent (HLGToLinearWithOOTF in GammaCorrectionCommon.ush) is a
 * reference-display specialization of this function with PeakLuminance=1000 and
 * no black lift.
 *
 * @param EncodedSceneReferred Encoded HLG signal.
 * @param PeakLuminance Display peak luminance in cd/m2 (default 1000).
 * @param BlackLuminance Display black luminance in cd/m2 (default 0).
 * @return Display-referred linear color in cd/m2.
 */
CORE_API FLinearColor DecodeHLG_2100(const FLinearColor& EncodedSceneReferred, const float PeakLuminance = 1000.0f, const float BlackLuminance = 0.0f);

/**
 * Direction in which a reference-white normalization scale is applied relative to a
 * transfer function. Selects between the decode scale (transfer-function linear
 * -> UE scene-linear) and the encode scale (UE scene-linear -> transfer-function
 * linear).
 */
enum class EReferenceWhiteDirection : uint8
{
	/** Apply to the output of the inverse transfer function: transfer_linear * scale = UE_scene_linear. */
	Decode = 0,
	/** Apply to the input of the transfer function: UE_scene_linear * scale = transfer_linear. */
	Encode = 1,
};

/**
 * Resolve an (EEncoding, EReferenceWhite, Direction) triple to the linear-side scalar
 * that normalizes between UE scene-linear (diffuse white = 1.0) and the transfer
 * function's native linear space.
 *
 * Reference-white normalization only applies to the HDR transfer functions (ST2084, HLG);
 * any other encoding returns 1.0 regardless of direction.
 */
CORE_API float GetReferenceWhiteLinearScale(EEncoding Encoding, EReferenceWhite ReferenceWhite, EReferenceWhiteDirection Direction = EReferenceWhiteDirection::Decode);

/** Get the encode function that matches the encoding type. */
CORE_API TFunction<float(float)> GetEncodeFunction(EEncoding Encoding);

/** Get the decode function that matches the encoding type. */
CORE_API TFunction<float(float)> GetDecodeFunction(EEncoding Encoding);

/** Encode a value based on the specified encoding type. Note: Less optimal due to function pointer call. */
CORE_API float Encode(EEncoding Encoding, float Value);

/** Decode a value based on the specified encoding type. Note: Less optimal due to function pointer call. */
CORE_API float Decode(EEncoding Encoding, float Value);

/** Get the encode function that matches the encoding type. */
CORE_API TFunction<FLinearColor(const FLinearColor&)> GetColorEncodeFunction(EEncoding Encoding);

/** Get the decode function that matches the encoding type. */
CORE_API TFunction<FLinearColor(const FLinearColor&)> GetColorDecodeFunction(EEncoding Encoding);

/** Encode a color based on the specified encoding type. Note: Less optimal due to function pointer call. */
CORE_API FLinearColor Encode(EEncoding Encoding, const FLinearColor& Color);

/** Decode a color based on the specified encoding type. Note: Less optimal due to function pointer call. */
CORE_API FLinearColor Decode(EEncoding Encoding, const FLinearColor& Color);

} } // end namespace UE::Color
