// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/NumericLimits.h"
#include "Memory/CompositeBuffer.h"
#include "Memory/MemoryFwd.h"
#include "Memory/SharedBuffer.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/RemoveReference.h"
#include "Templates/UnrealTemplate.h"

class FArchive;
struct FIoHash;

namespace FOodleDataCompression { enum class ECompressionLevel : int8; }
namespace FOodleDataCompression { enum class ECompressor : uint8; }

namespace UE::CompressedBuffer::Private { struct FHeader; }

using ECompressedBufferCompressionLevel = FOodleDataCompression::ECompressionLevel;
using ECompressedBufferCompressor = FOodleDataCompression::ECompressor;

enum class ECompressedBufferDecompressFlags : uint32
{
	None = 0,

	/**
	 * Decompress each block to an intermediate buffer before copying it to the target address.
	 *
	 * Use this flag to maintain performance when decompressing to uncached or write-combined memory.
	 */
	IntermediateBuffer = 1 << 0,
};

ENUM_CLASS_FLAGS(ECompressedBufferDecompressFlags);

/**
 * A compressed buffer stores compressed data in a self-contained format.
 *
 * A buffer is self-contained in the sense that it can be decompressed without external knowledge
 * of the compression format or the size of the raw data.
 *
 * The buffer may be partially decompressed using FCompressedBufferReader.
 *
 * @see FCompressedBufferReader
 */
class FCompressedBuffer
{
public:
	/**
	 * Compress the buffer using a balanced level of compression.
	 *
	 * @return An owned compressed buffer, or null on error.
	 */
	[[nodiscard]] CORE_API static FCompressedBuffer Compress(const FCompositeBuffer& RawData);
	[[nodiscard]] CORE_API static FCompressedBuffer Compress(const FSharedBuffer& RawData);

	/**
	 * Compress the buffer using the specified compressor and compression level.
	 *
	 * Data that does not compress will be return uncompressed, as if with level None.
	 *
	 * @note Using a level of None will return a buffer that references owned raw data.
	 *
	 * @param RawData            The raw data to be compressed.
	 * @param Compressor         The compressor to encode with. May use NotSet if level is None.
	 * @param CompressionLevel   The compression level to encode with.
	 * @param BlockSize          The power-of-two block size to encode raw data in. 0 is default.
	 * @return An owned compressed buffer, or null on error.
	 */
	[[nodiscard]] CORE_API static FCompressedBuffer Compress(
		const FCompositeBuffer& RawData,
		ECompressedBufferCompressor Compressor,
		ECompressedBufferCompressionLevel CompressionLevel,
		uint64 BlockSize = 0);
	[[nodiscard]] CORE_API static FCompressedBuffer Compress(
		const FSharedBuffer& RawData,
		ECompressedBufferCompressor Compressor,
		ECompressedBufferCompressionLevel CompressionLevel,
		uint64 BlockSize = 0);

	/**
	 * Construct from a compressed buffer previously created by Compress().
	 *
	 * @return A compressed buffer, or null on error, such as an invalid format or corrupt header.
	 */
	[[nodiscard]] CORE_API static FCompressedBuffer FromCompressed(const FCompositeBuffer& CompressedData);
	[[nodiscard]] CORE_API static FCompressedBuffer FromCompressed(FCompositeBuffer&& CompressedData);
	[[nodiscard]] CORE_API static FCompressedBuffer FromCompressed(const FSharedBuffer& CompressedData);
	[[nodiscard]] CORE_API static FCompressedBuffer FromCompressed(FSharedBuffer&& CompressedData);

	/**
	 * Load a compressed buffer from an archive, as saved by Save().
	 *
	 * The entire compressed buffer will be loaded from the archive before this function returns.
	 * Prefer to use FCompressedBufferReader to stream from an archive when the compressed buffer
	 * does not need to be fully loaded into memory.
	 *
	 * @return A compressed buffer, or null on error. Ar.IsError() will be true on error.
	 */
	[[nodiscard]] CORE_API static FCompressedBuffer Load(FArchive& Ar);

	/** Save the compressed buffer to an archive. */
	CORE_API void Save(FArchive& Ar) const;

	/** Reset this to null. */
	inline void Reset() { CompressedData.Reset(); }

	/** Returns true if the compressed buffer is not null. */
	[[nodiscard]] inline explicit operator bool() const { return !IsNull(); }

	/** Returns true if the compressed buffer is null. */
	[[nodiscard]] inline bool IsNull() const { return CompressedData.IsNull(); }

	/** Returns true if the composite buffer is owned. */
	[[nodiscard]] inline bool IsOwned() const { return CompressedData.IsOwned(); }

	/** Returns a copy of the compressed buffer that owns its underlying memory. */
	[[nodiscard]] inline FCompressedBuffer MakeOwned() const & { return FromCompressed(CompressedData.MakeOwned()); }
	[[nodiscard]] inline FCompressedBuffer MakeOwned() && { return FromCompressed(MoveTemp(CompressedData).MakeOwned()); }

	/** Returns a composite buffer containing the compressed data. May be null. May not be owned. */
	[[nodiscard]] inline const FCompositeBuffer& GetCompressed() const & { return CompressedData; }
	[[nodiscard]] inline FCompositeBuffer GetCompressed() && { return MoveTemp(CompressedData); }

	/** Returns the size of the compressed data. Zero on error or if this is null. */
	[[nodiscard]] CORE_API uint64 GetCompressedSize() const;

	/** Returns the size of the raw data. Zero on error or if this is empty or null. */
	[[nodiscard]] CORE_API uint64 GetRawSize() const;

	/** Returns the hash of the raw data. Zero on error or if this is null. */
	[[nodiscard]] CORE_API FIoHash GetRawHash() const;

	/**
	 * Returns the compressor and compression level used by this buffer.
	 *
	 * The compressor and compression level may differ from those specified when creating the buffer
	 * because an incompressible buffer is stored with no compression. Parameters cannot be accessed
	 * if this is null or uses a method other than Oodle, in which case this returns false.
	 *
	 * @return True if parameters were written, otherwise false.
	 */
	[[nodiscard]] CORE_API bool TryGetCompressParameters(
		ECompressedBufferCompressor& OutCompressor,
		ECompressedBufferCompressionLevel& OutCompressionLevel,
		uint64& OutBlockSize) const;

	/**
	 * Decompress into a memory view that is exactly equal to the raw size.
	 *
	 * @return True if the requested range was decompressed, otherwise false.
	 */
	[[nodiscard]] CORE_API bool TryDecompressTo(FMutableMemoryView RawView,
		ECompressedBufferDecompressFlags Flags = ECompressedBufferDecompressFlags::None) const;

	/**
	 * Decompress into an owned buffer.
	 *
	 * @return An owned buffer containing the raw data, or null on error.
	 */
	[[nodiscard]] CORE_API FSharedBuffer Decompress() const;
	[[nodiscard]] CORE_API FCompositeBuffer DecompressToComposite() const;

	/** A null compressed buffer. */
	CORE_API static const FCompressedBuffer Null;

private:
	FCompositeBuffer CompressedData;
};

CORE_API FArchive& operator<<(FArchive& Ar, FCompressedBuffer& Buffer);

namespace UE::CompressedBuffer::Private
{

/** A reusable context for the compressed buffer decoder. */
struct FDecoderContext
{
	/** The offset in the source at which the compressed buffer begins, otherwise MAX_uint64. */
	uint64 HeaderOffset = MAX_uint64;
	/** The size of the header if known, otherwise 0. */
	uint64 HeaderSize = 0;
	/** The CRC-32 from the header, otherwise 0. */
	uint32 HeaderCrc32 = 0;
	/** Index of the block stored in RawBlock, otherwise MAX_uint32. */
	uint32 RawBlockIndex = MAX_uint32;

	/** A buffer used to store the header when HeaderOffset is not MAX_uint64. */
	FUniqueBuffer Header;
	/** A buffer used to store a raw block when a partial block read is requested. */
	FUniqueBuffer RawBlock;
	/** A buffer used to store a compressed block when it was not in contiguous memory. */
	FUniqueBuffer CompressedBlock;
};

} // UE::CompressedBuffer::Private

/**
 * A type that stores the state needed to decompress a compressed buffer.
 *
 * The compressed buffer can be in memory or can be loaded from a seekable archive.
 *
 * The reader can be reused across multiple source buffers, which allows its temporary buffers to
 * be reused if they are the right size.
 *
 * It is only safe to use the reader from one thread at a time.
 *
 * @see FCompressedBuffer
 */
class FCompressedBufferReader
{
public:
	/** Construct a reader with no source. */
	FCompressedBufferReader() = default;

	/** Construct a reader that will read from an archive as needed. */
	CORE_API explicit FCompressedBufferReader(FArchive& Archive);

	/** Construct a reader from an in-memory compressed buffer. */
	CORE_API explicit FCompressedBufferReader(const FCompressedBuffer& Buffer);

	/** Release any temporary buffers that have been allocated by the reader. */
	CORE_API void ResetBuffers();

	/** Clears the reference to the source without releasing temporary buffers. */
	CORE_API void ResetSource();

	CORE_API void SetSource(FArchive& Archive);
	CORE_API void SetSource(const FCompressedBuffer& Buffer);

	[[nodiscard]] inline bool HasSource() const { return SourceArchive || SourceBuffer; }

	/** Returns the size of the compressed data. Zero on error. */
	[[nodiscard]] CORE_API uint64 GetCompressedSize();

	/** Returns the size of the raw data. Zero on error. */
	[[nodiscard]] CORE_API uint64 GetRawSize();

	/** Returns the hash of the raw data. Zero on error. */
	[[nodiscard]] CORE_API FIoHash GetRawHash();

	/**
	 * Returns the compressor and compression level used by this buffer.
	 *
	 * The compressor and compression level may differ from those specified when creating the buffer
	 * because an incompressible buffer is stored with no compression. Parameters cannot be accessed
	 * if this is null or uses a method other than Oodle, in which case this returns false.
	 *
	 * @return True if parameters were written, otherwise false.
	 */
	[[nodiscard]] CORE_API bool TryGetCompressParameters(
		ECompressedBufferCompressor& OutCompressor,
		ECompressedBufferCompressionLevel& OutCompressionLevel,
		uint64& OutBlockSize);

	/**
	 * Decompress into a memory view that is less than or equal to the available raw size.
	 *
	 * @param RawView     The view to write to. The size to read is equal to the view size.
	 * @param RawOffset   The offset into the raw data from which to decompress.
	 * @return True if the requested range was decompressed, otherwise false.
	 */
	[[nodiscard]] CORE_API bool TryDecompressTo(FMutableMemoryView RawView, uint64 RawOffset = 0,
		ECompressedBufferDecompressFlags Flags = ECompressedBufferDecompressFlags::None);

	/**
	 * Decompress into an owned buffer.
	 *
	 * RawOffset must be at most the raw buffer size. RawSize may be MAX_uint64 to read the whole
	 * buffer from RawOffset, and must otherwise fit within the bounds of the buffer.
	 *
	 * @param RawOffset   The offset into the raw data from which to decompress.
	 * @param RawSize     The size of the raw data to read from the offset.
	 * @return An owned buffer containing the raw data, or null on error.
	 */
	[[nodiscard]] CORE_API FSharedBuffer Decompress(uint64 RawOffset = 0, uint64 RawSize = MAX_uint64);
	[[nodiscard]] CORE_API FCompositeBuffer DecompressToComposite(uint64 RawOffset = 0, uint64 RawSize = MAX_uint64);

private:
	bool TryReadHeader(UE::CompressedBuffer::Private::FHeader& OutHeader, FMemoryView& OutHeaderView);

	FArchive* SourceArchive = nullptr;
	const FCompressedBuffer* SourceBuffer = nullptr;
	UE::CompressedBuffer::Private::FDecoderContext Context;
};

/** A type that sets the source of a reader upon construction and resets it upon destruction. */
class FCompressedBufferReaderSourceScope
{
public:
	inline FCompressedBufferReaderSourceScope(FCompressedBufferReader& InReader, FArchive& InArchive)
		: Reader(InReader)
	{
		Reader.SetSource(InArchive);
	}

	inline FCompressedBufferReaderSourceScope(FCompressedBufferReader& InReader, const FCompressedBuffer& InBuffer)
		: Reader(InReader)
	{
		Reader.SetSource(InBuffer);
	}

	inline ~FCompressedBufferReaderSourceScope()
	{
		Reader.ResetSource();
	}

private:
	FCompressedBufferReader& Reader;
};
