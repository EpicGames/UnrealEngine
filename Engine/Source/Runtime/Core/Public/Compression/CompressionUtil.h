// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"

class FArchive;
class FCustomVersionContainer;
class FLinker;
class FName;

namespace FCompressionUtil
{


	void CORE_API SerializeCompressorName(FArchive & Archive,FName & Compressor);


	/** Logs or writes to an array of strings a memory region as a hex dump
	 * For logging, log category: LogCore, level: Display.
	 *  Bytes, BytesNum are used to designate the overall memory region of interest (e.g. a file in memory), but only bytes from OffsetStart to OffsetEnd will be logged out,
	 * and current offset will be printed at each line's start.
	 * 
	 * @param Bytes memory location of the data 
	 * @param BytesNum total size of the data
	 * @param OffsetStart offset to first byte to be logged out (pass 0 if you want to log out whole region)
	 * @param OffsetEnd offset to the end of log window (pass BytesNum if you want to log out whole region)
	 * @param BytesPerLine how many bytes to print per line of hex dump
	 */
	TArray<FString> CORE_API HexDumpLines(const uint8* Bytes, int64 BytesNum, int64 OffsetStart, int64 OffsetEnd, int32 BytesPerLine = 32);
	void CORE_API LogHexDump(const uint8* Bytes, int64 BytesNum, int64 OffsetStart, int64 OffsetEnd, int32 BytesPerLine = 32);
};