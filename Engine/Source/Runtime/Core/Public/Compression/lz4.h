/*
 *  LZ4 - Fast LZ compression algorithm
 *  Header File
 *  Copyright (C) 2011-2020, Yann Collet.

   BSD 2-Clause License (http://www.opensource.org/licenses/bsd-license.php)

   Redistribution and use in source and binary forms, with or without
   modification, are permitted provided that the following conditions are
   met:

       * Redistributions of source code must retain the above copyright
   notice, this list of conditions and the following disclaimer.
       * Redistributions in binary form must reproduce the above
   copyright notice, this list of conditions and the following disclaimer
   in the documentation and/or other materials provided with the
   distribution.

   THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS
   "AS IS" AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT
   LIMITED TO, THE IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR
   A PARTICULAR PURPOSE ARE DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT
   OWNER OR CONTRIBUTORS BE LIABLE FOR ANY DIRECT, INDIRECT, INCIDENTAL,
   SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES (INCLUDING, BUT NOT
   LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES; LOSS OF USE,
   DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON ANY
   THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
   (INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE
   OF THIS SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.

   You can contact the author at :
    - LZ4 homepage : http://www.lz4.org
    - LZ4 source repository : https://github.com/lz4/lz4
*/


// HEADER_UNIT_SKIP - Not compatible

#include "HAL/PlatformMemory.h"
#if defined (__cplusplus)
extern "C" {
#endif

#ifndef LZ4_H_2983827168210
#define LZ4_H_2983827168210

/* --- Dependency --- */


/**
  Introduction

  LZ4 is lossless compression algorithm, providing compression speed >500 MB/s per core,
  scalable with multi-cores CPU. It features an extremely fast decoder, with speed in
  multiple GB/s per core, typically reaching RAM speed limits on multi-core systems.

  The LZ4 compression library provides in-memory compression and decompression functions.
  It gives full buffer control to user.
  Compression can be done in:
    - a single step (described as Simple Functions)
    - a single step, reusing a context (described in Advanced Functions)
    - unbounded multiple steps (described as Streaming compression)

  lz4.h generates and decodes LZ4-compressed blocks (doc/lz4_Block_format.md).
  Decompressing such a compressed block requires additional metadata.
  Exact metadata depends on exact decompression function.
  For the typical case of LZ4_decompress_safe(),
  metadata includes block's compressed size, and maximum bound of decompressed size.
  Each application is free to encode and pass such metadata in whichever way it wants.

  lz4.h only handle blocks, it can not generate Frames.

  Blocks are different from Frames (doc/lz4_Frame_format.md).
  Frames bundle both blocks and metadata in a specified manner.
  Embedding metadata is required for compressed data to be self-contained and portable.
  Frame format is delivered through a companion API, declared in lz4frame.h.
  The `lz4` CLI can only manage frames.
*/

/*^***************************************************************
*  Export parameters
*****************************************************************/

#define LZ4LIB_API CORE_API

/*! LZ4_FREESTANDING :
 *  When this macro is set to 1, it enables "freestanding mode" that is
 *  suitable for typical freestanding environment which doesn't support
 *  standard C library.
 *
 *  - LZ4_FREESTANDING is a compile-time switch.
 *  - It requires the following macros to be defined:
 *    LZ4_memcpy, LZ4_memmove, LZ4_memset.
 *  - It only enables LZ4/HC functions which don't use heap.
 *    All LZ4F_* functions are not supported.
 *  - See tests/freestanding.c to check its basic setup.
 */
#if defined(LZ4_FREESTANDING) && (LZ4_FREESTANDING == 1)
#  define LZ4_HEAPMODE 0
#  define LZ4HC_HEAPMODE 0
#  define LZ4_STATIC_LINKING_ONLY_DISABLE_MEMORY_ALLOCATION 1
#  if !defined(LZ4_memcpy)
#    error "LZ4_FREESTANDING requires macro 'LZ4_memcpy'."
#  endif
#  if !defined(LZ4_memset)
#    error "LZ4_FREESTANDING requires macro 'LZ4_memset'."
#  endif
#  if !defined(LZ4_memmove)
#    error "LZ4_FREESTANDING requires macro 'LZ4_memmove'."
#  endif
#elif ! defined(LZ4_FREESTANDING)
#  define LZ4_FREESTANDING 0
#endif


/*------   Version   ------*/
#define LZ4_VERSION_MAJOR    1    /* for breaking interface changes  */
#define LZ4_VERSION_MINOR    9    /* for new (non-breaking) interface capabilities */
#define LZ4_VERSION_RELEASE  4    /* for tweaks, bug-fixes, or development */

#define LZ4_VERSION_NUMBER (LZ4_VERSION_MAJOR *100*100 + LZ4_VERSION_MINOR *100 + LZ4_VERSION_RELEASE)

#define LZ4_LIB_VERSION LZ4_VERSION_MAJOR.LZ4_VERSION_MINOR.LZ4_VERSION_RELEASE
#define LZ4_QUOTE(str) #str
#define LZ4_EXPAND_AND_QUOTE(str) LZ4_QUOTE(str)
#define LZ4_VERSION_STRING LZ4_EXPAND_AND_QUOTE(LZ4_LIB_VERSION)  /* requires v1.7.3+ */

LZ4LIB_API int LZ4_versionNumber (void);  /**< library version number; useful to check dll version; requires v1.3.0+ */
LZ4LIB_API const char* LZ4_versionString (void);   /**< library version string; useful to check dll version; requires v1.7.5+ */


/*-************************************
*  Tuning parameter
**************************************/
#define LZ4_MEMORY_USAGE_MIN 10
#define LZ4_MEMORY_USAGE_DEFAULT 14
#define LZ4_MEMORY_USAGE_MAX 20

/*!
 * LZ4_MEMORY_USAGE :
 * Memory usage formula : N->2^N Bytes (examples : 10 -> 1KB; 12 -> 4KB ; 16 -> 64KB; 20 -> 1MB; )
 * Increasing memory usage improves compression ratio, at the cost of speed.
 * Reduced memory usage may improve speed at the cost of ratio, thanks to better cache locality.
 * Default value is 14, for 16KB, which nicely fits into Intel x86 L1 cache
 */
#ifndef LZ4_MEMORY_USAGE
# define LZ4_MEMORY_USAGE LZ4_MEMORY_USAGE_DEFAULT
#endif

#if (LZ4_MEMORY_USAGE < LZ4_MEMORY_USAGE_MIN)
#  error "LZ4_MEMORY_USAGE is too small !"
#endif

#if (LZ4_MEMORY_USAGE > LZ4_MEMORY_USAGE_MAX)
#  error "LZ4_MEMORY_USAGE is too large !"
#endif

/*-************************************
*  Simple Functions
**************************************/
/*! LZ4_compress_default() :
 *  Compresses 'srcSize' bytes from buffer 'src'
 *  into already allocated 'dst' buffer of size 'dstCapacity'.
 *  Compression is guaranteed to succeed if 'dstCapacity' >= LZ4_compressBound(srcSize).
 *  It also runs faster, so it's a recommended setting.
 *  If the function cannot compress 'src' into a more limited 'dst' budget,
 *  compression stops *immediately*, and the function result is zero.
 *  In which case, 'dst' content is undefined (invalid).
 *      srcSize : max supported value is LZ4_MAX_INPUT_SIZE.
 *      dstCapacity : size of buffer 'dst' (which must be already allocated)
 *     @return  : the number of bytes written into buffer 'dst' (necessarily <= dstCapacity)
 *                or 0 if compression fails
 * Note : This function is protected against buffer overflow scenarios (never writes outside 'dst' buffer, nor read outside 'source' buffer).
 */
LZ4LIB_API int LZ4_compress_default(const char* src, char* dst, int srcSize, int dstCapacity);

/*! LZ4_decompress_safe() :
 *  compressedSize : is the exact complete size of the compressed block.
 *  dstCapacity : is the size of destination buffer (which must be already allocated), presumed an upper bound of decompressed size.
 * @return : the number of bytes decompressed into destination buffer (necessarily <= dstCapacity)
 *           If destination buffer is not large enough, decoding will stop and output an error code (negative value).
 *           If the source stream is detected malformed, the function will stop decoding and return a negative result.
 * Note 1 : This function is protected against malicious data packets :
 *          it will never writes outside 'dst' buffer, nor read outside 'source' buffer,
 *          even if the compressed block is maliciously modified to order the decoder to do these actions.
 *          In such case, the decoder stops immediately, and considers the compressed block malformed.
 * Note 2 : compressedSize and dstCapacity must be provided to the function, the compressed block does not contain them.
 *          The implementation is free to send / store / derive this information in whichever way is most beneficial.
 *          If there is a need for a different format which bundles together both compressed data and its metadata, consider looking at lz4frame.h instead.
 */
LZ4LIB_API int LZ4_decompress_safe (const char* src, char* dst, int compressedSize, int dstCapacity);


/*-************************************
*  Advanced Functions
**************************************/
#define LZ4_MAX_INPUT_SIZE        0x7E000000   /* 2 113 929 216 bytes */
#define LZ4_COMPRESSBOUND(isize)  ((unsigned)(isize) > (unsigned)LZ4_MAX_INPUT_SIZE ? 0 : (isize) + ((isize)/255) + 16)

/*! LZ4_compressBound() :
    Provides the maximum size that LZ4 compression may output in a "worst case" scenario (input data not compressible)
    This function is primarily useful for memory allocation purposes (destination buffer size).
    Macro LZ4_COMPRESSBOUND() is also provided for compilation-time evaluation (stack memory allocation for example).
    Note that LZ4_compress_default() compresses faster when dstCapacity is >= LZ4_compressBound(srcSize)
        inputSize  : max supported value is LZ4_MAX_INPUT_SIZE
        return : maximum output size in a "worst case" scenario
              or 0, if input size is incorrect (too large or negative)
*/
LZ4LIB_API int LZ4_compressBound(int inputSize);

/*! LZ4_compress_fast() :
    Same as LZ4_compress_default(), but allows selection of "acceleration" factor.
    The larger the acceleration value, the faster the algorithm, but also the lesser the compression.
    It's a trade-off. It can be fine tuned, with each successive value providing roughly +~3% to speed.
    An acceleration value of "1" is the same as regular LZ4_compress_default()
    Values <= 0 will be replaced by LZ4_ACCELERATION_DEFAULT (currently == 1, see lz4.c).
    Values > LZ4_ACCELERATION_MAX will be replaced by LZ4_ACCELERATION_MAX (currently == 65537, see lz4.c).
*/
LZ4LIB_API int LZ4_compress_fast (const char* src, char* dst, int srcSize, int dstCapacity, int acceleration);


/*! LZ4_compress_fast_extState() :
 *  Same as LZ4_compress_fast(), using an externally allocated memory space for its state.
 *  Use LZ4_sizeofState() to know how much memory must be allocated,
 *  and allocate it on 8-bytes boundaries (using `malloc()` typically).
 *  Then, provide this buffer as `void* state` to compression function.
 */
LZ4LIB_API int LZ4_sizeofState(void);
LZ4LIB_API int LZ4_compress_fast_extState (void* state, const char* src, char* dst, int srcSize, int dstCapacity, int acceleration);


/*! LZ4_compress_destSize() :
 *  Reverse the logic : compresses as much data as possible from 'src' buffer
 *  into already allocated buffer 'dst', of size >= 'targetDestSize'.
 *  This function either compresses the entire 'src' content into 'dst' if it's large enough,
 *  or fill 'dst' buffer completely with as much data as possible from 'src'.
 *  note: acceleration parameter is fixed to "default".
 *
 * *srcSizePtr : will be modified to indicate how many bytes where read from 'src' to fill 'dst'.
 *               New value is necessarily <= input value.
 * @return : Nb bytes written into 'dst' (necessarily <= targetDestSize)
 *           or 0 if compression fails.
 *
 * Note : from v1.8.2 to v1.9.1, this function had a bug (fixed un v1.9.2+):
 *        the produced compressed content could, in specific circumstances,
 *        require to be decompressed into a destination buffer larger
 *        by at least 1 byte than the content to decompress.
 *        If an application uses `LZ4_compress_destSize()`,
 *        it's highly recommended to update liblz4 to v1.9.2 or better.
 *        If this can't be done or ensured,
 *        the receiving decompression function should provide
 *        a dstCapacity which is > decompressedSize, by at least 1 byte.
 *        See https://github.com/lz4/lz4/issues/859 for details
 */
LZ4LIB_API int LZ4_compress_destSize (const char* src, char* dst, int* srcSizePtr, int targetDstSize);


/*! LZ4_decompress_safe_partial() :
 *  Decompress an LZ4 compressed block, of size 'srcSize' at position 'src',
 *  into destination buffer 'dst' of size 'dstCapacity'.
 *  Up to 'targetOutputSize' bytes will be decoded.
 *  The function stops decoding on reaching this objective.
 *  This can be useful to boost performance
 *  whenever only the beginning of a block is required.
 *
 * @return : the number of bytes decoded in `dst` (necessarily <= targetOutputSize)
 *           If source stream is detected malformed, function returns a negative result.
 *
 *  Note 1 : @return can be < targetOutputSize, if compressed block contains less data.
 *
 *  Note 2 : targetOutputSize must be <= dstCapacity
 *
 *  Note 3 : this function effectively stops decoding on reaching targetOutputSize,
 *           so dstCapacity is kind of redundant.
 *           This is because in older versions of this function,
 *           decoding operation would still write complete sequences.
 *           Therefore, there was no guarantee that it would stop writing at exactly targetOutputSize,
 *           it could write more bytes, though only up to dstCapacity.
 *           Some "margin" used to be required for this operation to work properly.
 *           Thankfully, this is no longer necessary.
 *           The function nonetheless keeps the same signature, in an effort to preserve API compatibility.
 *
 *  Note 4 : If srcSize is the exact size of the block,
 *           then targetOutputSize can be any value,
 *           including larger than the block's decompressed size.
 *           The function will, at most, generate block's decompressed size.
 *
 *  Note 5 : If srcSize is _larger_ than block's compressed size,
 *           then targetOutputSize **MUST** be <= block's decompressed size.
 *           Otherwise, *silent corruption will occur*.
 */
LZ4LIB_API int LZ4_decompress_safe_partial (const char* src, char* dst, int srcSize, int targetOutputSize, int dstCapacity);


/*-*********************************************
*  Streaming Compression Functions
***********************************************/
typedef union LZ4_stream_u LZ4_stream_t;  /* incomplete type (defined later) */

/**
 Note about RC_INVOKED

 - RC_INVOKED is predefined symbol of rc.exe (the resource compiler which is part of MSVC/Visual Studio).
   https://docs.microsoft.com/en-us/windows/win32/menurc/predefined-macros

 - Since rc.exe is a legacy compiler, it truncates long symbol (> 30 chars)
   and reports warning "RC4011: identifier truncated".

 - To eliminate the warning, we surround long preprocessor symbol with
   "#if !defined(RC_INVOKED) ... #endif" block that means
   "skip this block when rc.exe is trying to read it".
*/
#if !defined(RC_INVOKED) /* https://docs.microsoft.com/en-us/windows/win32/menurc/predefined-macros */
#if !defined(LZ4_STATIC_LINKING_ONLY_DISABLE_MEMORY_ALLOCATION)
LZ4LIB_API LZ4_stream_t* LZ4_createStream(void);
LZ4LIB_API int           LZ4_freeStream (LZ4_stream_t* streamPtr);
#endif /* !defined(LZ4_STATIC_LINKING_ONLY_DISABLE_MEMORY_ALLOCATION) */
#endif

/*! LZ4_resetStream_fast() : v1.9.0+
 *  Use this to prepare an LZ4_stream_t for a new chain of dependent blocks
 *  (e.g., LZ4_compress_fast_continue()).
 *
 *  An LZ4_stream_t must be initialized once before usage.
 *  This is automatically done when created by LZ4_createStream().
 *  However, should the LZ4_stream_t be simply declared on stack (for example),
 *  it's necessary to initialize it first, using LZ4_initStream().
 *
 *  After init, start any new stream with LZ4_resetStream_fast().
 *  A same LZ4_stream_t can be re-used multiple times consecutively
 *  and compress multiple streams,
 *  provided that it starts each new stream with LZ4_resetStream_fast().
 *
 *  LZ4_resetStream_fast() is much faster than LZ4_initStream(),
 *  but is not compatible with memory regions containing garbage data.
 *
 *  Note: it's only useful to call LZ4_resetStream_fast()
 *        in the context of streaming compression.
 *        The *extState* functions perform their own resets.
 *        Invoking LZ4_resetStream_fast() before is redundant, and even counterproductive.
 */
LZ4LIB_API void LZ4_resetStream_fast (LZ4_stream_t* streamPtr);

/*! LZ4_loadDict() :
 *  Use this function to reference a static dictionary into LZ4_stream_t.
 *  The dictionary must remain available during compression.
 *  LZ4_loadDict() triggers a reset, so any previous data will be forgotten.
 *  The same dictionary will have to be loaded on decompression side for successful decoding.
 *  Dictionary are useful for better compression of small data (KB range).
 *  While LZ4 accept any input as dictionary,
 *  results are generally better when using Zstandard's Dictionary Builder.
 *  Loading a size of 0 is allowed, and is the same as reset.
 * @return : loaded dictionary size, in bytes (necessarily <= 64 KB)
 */
LZ4LIB_API int LZ4_loadDict (LZ4_stream_t* streamPtr, const char* dictionary, int dictSize);

/*! LZ4_compress_fast_continue() :
 *  Compress 'src' content using data from previously compressed blocks, for better compression ratio.
 * 'dst' buffer must be already allocated.
 *  If dstCapacity >= LZ4_compressBound(srcSize), compression is guaranteed to succeed, and runs faster.
 *
 * @return : size of compressed block
 *           or 0 if there is an error (typically, cannot fit into 'dst').
 *
 *  Note 1 : Each invocation to LZ4_compress_fast_continue() generates a new block.
 *           Each block has precise boundaries.
 *           Each block must be decompressed separately, calling LZ4_decompress_*() with relevant metadata.
 *           It's not possible to append blocks together and expect a single invocation of LZ4_decompress_*() to decompress them together.
 *
 *  Note 2 : The previous 64KB of source data is __assumed__ to remain present, unmodified, at same address in memory !
 *
 *  Note 3 : When input is structured as a double-buffer, each buffer can have any size, including < 64 KB.
 *           Make sure that buffers are separated, by at least one byte.
 *           This construction ensures that each block only depends on previous block.
 *
 *  Note 4 : If input buffer is a ring-buffer, it can have any size, including < 64 KB.
 *
 *  Note 5 : After an error, the stream status is undefined (invalid), it can only be reset or freed.
 */
LZ4LIB_API int LZ4_compress_fast_continue (LZ4_stream_t* streamPtr, const char* src, char* dst, int srcSize, int dstCapacity, int acceleration);

/*! LZ4_saveDict() :
 *  If last 64KB data cannot be guaranteed to remain available at its current memory location,
 *  save it into a safer place (char* safeBuffer).
 *  This is schematically equivalent to a memcpy() followed by LZ4_loadDict(),
 *  but is much faster, because LZ4_saveDict() doesn't need to rebuild tables.
 * @return : saved dictionary size in bytes (necessarily <= maxDictSize), or 0 if error.
 */
LZ4LIB_API int LZ4_saveDict (LZ4_stream_t* streamPtr, char* safeBuffer, int maxDictSize);


/*-**********************************************
*  Streaming Decompression Functions
*  Bufferless synchronous API
************************************************/
typedef union LZ4_streamDecode_u LZ4_streamDecode_t;   /* tracking context */

/*! LZ4_createStreamDecode() and LZ4_freeStreamDecode() :
 *  creation / destruction of streaming decompression tracking context.
 *  A tracking context can be re-used multiple times.
 */
#if !defined(RC_INVOKED) /* https://docs.microsoft.com/en-us/windows/win32/menurc/predefined-macros */
#if !defined(LZ4_STATIC_LINKING_ONLY_DISABLE_MEMORY_ALLOCATION)
LZ4LIB_API LZ4_streamDecode_t* LZ4_createStreamDecode(void);
LZ4LIB_API int                 LZ4_freeStreamDecode (LZ4_streamDecode_t* LZ4_stream);
#endif /* !defined(LZ4_STATIC_LINKING_ONLY_DISABLE_MEMORY_ALLOCATION) */
#endif

/*! LZ4_setStreamDecode() :
 *  An LZ4_streamDecode_t context can be allocated once and re-used multiple times.
 *  Use this function to start decompression of a new stream of blocks.
 *  A dictionary can optionally be set. Use NULL or size 0 for a reset order.
 *  Dictionary is presumed stable : it must remain accessible and unmodified during next decompression.
 * @return : 1 if OK, 0 if error
 */
LZ4LIB_API int LZ4_setStreamDecode (LZ4_streamDecode_t* LZ4_streamDecode, const char* dictionary, int dictSize);

/*! LZ4_decoderRingBufferSize() : v1.8.2+
 *  Note : in a ring buffer scenario (optional),
 *  blocks are presumed decompressed next to each other
 *  up to the moment there is not enough remaining space for next block (remainingSize < maxBlockSize),
 *  at which stage it resumes from beginning of ring buffer.
 *  When setting such a ring buffer for streaming decompression,
 *  provides the minimum size of this ring buffer
 *  to be compatible with any source respecting maxBlockSize condition.
 * @return : minimum ring buffer size,
 *           or 0 if there is an error (invalid maxBlockSize).
 */
LZ4LIB_API int LZ4_decoderRingBufferSize(int maxBlockSize);
#define LZ4_DECODER_RING_BUFFER_SIZE(maxBlockSize) (65536 + 14 + (maxBlockSize))  /* for static allocation; maxBlockSize presumed valid */

/*! LZ4_decompress_*_continue() :
 *  These decoding functions allow decompression of consecutive blocks in "streaming" mode.
 *  A block is an unsplittable entity, it must be presented entirely to a decompression function.
 *  Decompression functions only accepts one block at a time.
 *  The last 64KB of previously decoded data *must* remain available and unmodified at the memory position where they were decoded.
 *  If less than 64KB of data has been decoded, all the data must be present.
 *
 *  Special : if decompression side sets a ring buffer, it must respect one of the following conditions :
 *  - Decompression buffer size is _at least_ LZ4_decoderRingBufferSize(maxBlockSize).
 *    maxBlockSize is the maximum size of any single block. It can have any value > 16 bytes.
 *    In which case, encoding and decoding buffers do not need to be synchronized.
 *    Actually, data can be produced by any source compliant with LZ4 format specification, and respecting maxBlockSize.
 *  - Synchronized mode :
 *    Decompression buffer size is _exactly_ the same as compression buffer size,
 *    and follows exactly same update rule (block boundaries at same positions),
 *    and decoding function is provided with exact decompressed size of each block (exception for last block of the stream),
 *    _then_ decoding & encoding ring buffer can have any size, including small ones ( < 64 KB).
 *  - Decompression buffer is larger than encoding buffer, by a minimum of maxBlockSize more bytes.
 *    In which case, encoding and decoding buffers do not need to be synchronized,
 *    and encoding ring buffer can have any size, including small ones ( < 64 KB).
 *
 *  Whenever these conditions are not possible,
 *  save the last 64KB of decoded data into a safe buffer where it can't be modified during decompression,
 *  then indicate where this data is saved using LZ4_setStreamDecode(), before decompressing next block.
*/
LZ4LIB_API int
LZ4_decompress_safe_continue (LZ4_streamDecode_t* LZ4_streamDecode,
                        const char* src, char* dst,
                        int srcSize, int dstCapacity);


/*! LZ4_decompress_*_usingDict() :
 *  These decoding functions work the same as
 *  a combination of LZ4_setStreamDecode() followed by LZ4_decompress_*_continue()
 *  They are stand-alone, and don't need an LZ4_streamDecode_t structure.
 *  Dictionary is presumed stable : it must remain accessible and unmodified during decompression.
 *  Performance tip : Decompression speed can be substantially increased
 *                    when dst == dictStart + dictSize.
 */
LZ4LIB_API int
LZ4_decompress_safe_usingDict(const char* src, char* dst,
                              int srcSize, int dstCapacity,
                              const char* dictStart, int dictSize);

LZ4LIB_API int
LZ4_decompress_safe_partial_usingDict(const char* src, char* dst,
                                      int compressedSize,
                                      int targetOutputSize, int maxOutputSize,
                                      const char* dictStart, int dictSize);

#endif /* LZ4_H_2983827168210 */


/*^*************************************
 * !!!!!!   STATIC LINKING ONLY   !!!!!!
 ***************************************/

/*-****************************************************************************
 * Experimental section
 *
 * Symbols declared in this section must be considered unstable. Their
 * signatures or semantics may change, or they may be removed altogether in the
 * future. They are therefore only safe to depend on when the caller is
 * statically linked against the library.
 *
 * To protect against unsafe usage, not only are the declarations guarded,
 * the definitions are hidden by default
 * when building LZ4 as a shared/dynamic library.
 *
 * In order to access these declarations,
 * define LZ4_STATIC_LINKING_ONLY in your application
 * before including LZ4's headers.
 *
 * In order to make their implementations accessible dynamically, you must
 * define LZ4_PUBLISH_STATIC_FUNCTIONS when building the LZ4 library.
 ******************************************************************************/

#ifdef LZ4_STATIC_LINKING_ONLY

#ifndef LZ4_STATIC_3504398509
#define LZ4_STATIC_3504398509

#ifdef LZ4_PUBLISH_STATIC_FUNCTIONS
#define LZ4LIB_STATIC_API LZ4LIB_API
#else
#define LZ4LIB_STATIC_API
#endif


/*! LZ4_compress_fast_extState_fastReset() :
 *  A variant of LZ4_compress_fast_extState().
 *
 *  Using this variant avoids an expensive initialization step.
 *  It is only safe to call if the state buffer is known to be correctly initialized already
 *  (see above comment on LZ4_resetStream_fast() for a definition of "correctly initialized").
 *  From a high level, the difference is that
 *  this function initializes the provided state with a call to something like LZ4_resetStream_fast()
 *  while LZ4_compress_fast_extState() starts with a call to LZ4_resetStream().
 */
LZ4LIB_STATIC_API int LZ4_compress_fast_extState_fastReset (void* state, const char* src, char* dst, int srcSize, int dstCapacity, int acceleration);

/*! LZ4_attach_dictionary() :
 *  This is an experimental API that allows
 *  efficient use of a static dictionary many times.
 *
 *  Rather than re-loading the dictionary buffer into a working context before
 *  each compression, or copying a pre-loaded dictionary's LZ4_stream_t into a
 *  working LZ4_stream_t, this function introduces a no-copy setup mechanism,
 *  in which the working stream references the dictionary stream in-place.
 *
 *  Several assumptions are made about the state of the dictionary stream.
 *  Currently, only streams which have been prepared by LZ4_loadDict() should
 *  be expected to work.
 *
 *  Alternatively, the provided dictionaryStream may be NULL,
 *  in which case any existing dictionary stream is unset.
 *
 *  If a dictionary is provided, it replaces any pre-existing stream history.
 *  The dictionary contents are the only history that can be referenced and
 *  logically immediately precede the data compressed in the first subsequent
 *  compression call.
 *
 *  The dictionary will only remain attached to the working stream through the
 *  first compression call, at the end of which it is cleared. The dictionary
 *  stream (and source buffer) must remain in-place / accessible / unchanged
 *  through the completion of the first compression call on the stream.
 */
LZ4LIB_STATIC_API void
LZ4_attach_dictionary(LZ4_stream_t* workingStream,
                const LZ4_stream_t* dictionaryStream);


/*! In-place compression and decompression
 *
 * It's possible to have input and output sharing the same buffer,
 * for highly constrained memory environments.
 * In both cases, it requires input to lay at the end of the buffer,
 * and decompression to start at beginning of the buffer.
 * Buffer size must feature some margin, hence be larger than final size.
 *
 * |<------------------------buffer--------------------------------->|
 *                             |<-----------compressed data--------->|
 * |<-----------decompressed size------------------>|
 *                                                  |<----margin---->|
 *
 * This technique is more useful for decompression,
 * since decompressed size is typically larger,
 * and margin is short.
 *
 * In-place decompression will work inside any buffer
 * which size is >= LZ4_DECOMPRESS_INPLACE_BUFFER_SIZE(decompressedSize).
 * This presumes that decompressedSize > compressedSize.
 * Otherwise, it means compression actually expanded data,
 * and it would be more efficient to store such data with a flag indicating it's not compressed.
 * This can happen when data is not compressible (already compressed, or encrypted).
 *
 * For in-place compression, margin is larger, as it must be able to cope with both
 * history preservation, requiring input data to remain unmodified up to LZ4_DISTANCE_MAX,
 * and data expansion, which can happen when input is not compressible.
 * As a consequence, buffer size requirements are much higher,
 * and memory savings offered by in-place compression are more limited.
 *
 * There are ways to limit this cost for compression :
 * - Reduce history size, by modifying LZ4_DISTANCE_MAX.
 *   Note that it is a compile-time constant, so all compressions will apply this limit.
 *   Lower values will reduce compression ratio, except when input_size < LZ4_DISTANCE_MAX,
 *   so it's a reasonable trick when inputs are known to be small.
 * - Require the compressor to deliver a "maximum compressed size".
 *   This is the `dstCapacity` parameter in `LZ4_compress*()`.
 *   When this size is < LZ4_COMPRESSBOUND(inputSize), then compression can fail,
 *   in which case, the return code will be 0 (zero).
 *   The caller must be ready for these cases to happen,
 *   and typically design a backup scheme to send data uncompressed.
 * The combination of both techniques can significantly reduce
 * the amount of margin required for in-place compression.
 *
 * In-place compression can work in any buffer
 * which size is >= (maxCompressedSize)
 * with maxCompressedSize == LZ4_COMPRESSBOUND(srcSize) for guaranteed compression success.
 * LZ4_COMPRESS_INPLACE_BUFFER_SIZE() depends on both maxCompressedSize and LZ4_DISTANCE_MAX,
 * so it's possible to reduce memory requirements by playing with them.
 */

#define LZ4_DECOMPRESS_INPLACE_MARGIN(compressedSize)          (((compressedSize) >> 8) + 32)
#define LZ4_DECOMPRESS_INPLACE_BUFFER_SIZE(decompressedSize)   ((decompressedSize) + LZ4_DECOMPRESS_INPLACE_MARGIN(decompressedSize))  /**< note: presumes that compressedSize < decompressedSize. note2: margin is overestimated a bit, since it could use compressedSize instead */

#ifndef LZ4_DISTANCE_MAX   /* history window size; can be user-defined at compile time */
#  define LZ4_DISTANCE_MAX 65535   /* set to maximum value by default */
#endif

#define LZ4_COMPRESS_INPLACE_MARGIN                           (LZ4_DISTANCE_MAX + 32)   /* LZ4_DISTANCE_MAX can be safely replaced by srcSize when it's smaller */
#define LZ4_COMPRESS_INPLACE_BUFFER_SIZE(maxCompressedSize)   ((maxCompressedSize) + LZ4_COMPRESS_INPLACE_MARGIN)  /**< maxCompressedSize is generally LZ4_COMPRESSBOUND(inputSize), but can be set to any lower value, with the risk that compression can fail (return code 0(zero)) */

#endif   /* LZ4_STATIC_3504398509 */
#endif   /* LZ4_STATIC_LINKING_ONLY */



#ifndef LZ4_H_98237428734687
#define LZ4_H_98237428734687

/*-************************************************************
 *  Private Definitions
 **************************************************************
 * Do not use these definitions directly.
 * They are only exposed to allow static allocation of `LZ4_stream_t` and `LZ4_streamDecode_t`.
 * Accessing members will expose user code to API and/or ABI break in future versions of the library.
 **************************************************************/
#define LZ4_HASHLOG   (LZ4_MEMORY_USAGE-2)
#define LZ4_HASHTABLESIZE (1 << LZ4_MEMORY_USAGE)
#define LZ4_HASH_SIZE_U32 (1 << LZ4_HASHLOG)       /* required as macro for static allocation */

#if defined(__cplusplus) || (defined (__STDC_VERSION__) && (__STDC_VERSION__ >= 199901L) /* C99 */)
  typedef  int8  LZ4_i8;
  typedef uint8  LZ4_byte;
  typedef uint16 LZ4_u16;
  typedef uint32 LZ4_u32;
#else
  typedef   signed char  LZ4_i8;
  typedef unsigned char  LZ4_byte;
  typedef unsigned short LZ4_u16;
  typedef unsigned int   LZ4_u32;
#endif

/*! LZ4_stream_t :
 *  Never ever use below internal definitions directly !
 *  These definitions are not API/ABI safe, and may change in future versions.
 *  If you need static allocation, declare or allocate an LZ4_stream_t object.
**/

typedef struct LZ4_stream_t_internal LZ4_stream_t_internal;
struct LZ4_stream_t_internal {
    LZ4_u32 hashTable[LZ4_HASH_SIZE_U32];
    const LZ4_byte* dictionary;
    const LZ4_stream_t_internal* dictCtx;
    LZ4_u32 currentOffset;
    LZ4_u32 tableType;
    LZ4_u32 dictSize;
    /* Implicit padding to ensure structure is aligned */
};

#define LZ4_STREAM_MINSIZE  ((1UL << LZ4_MEMORY_USAGE) + 32)  /* static size, for inter-version compatibility */
union LZ4_stream_u {
    char minStateSize[LZ4_STREAM_MINSIZE];
    LZ4_stream_t_internal internal_donotuse;
}; /* previously typedef'd to LZ4_stream_t */


/*! LZ4_initStream() : v1.9.0+
 *  An LZ4_stream_t structure must be initialized at least once.
 *  This is automatically done when invoking LZ4_createStream(),
 *  but it's not when the structure is simply declared on stack (for example).
 *
 *  Use LZ4_initStream() to properly initialize a newly declared LZ4_stream_t.
 *  It can also initialize any arbitrary buffer of sufficient size,
 *  and will @return a pointer of proper type upon initialization.
 *
 *  Note : initialization fails if size and alignment conditions are not respected.
 *         In which case, the function will @return NULL.
 *  Note2: An LZ4_stream_t structure guarantees correct alignment and size.
 *  Note3: Before v1.9.0, use LZ4_resetStream() instead
**/
LZ4LIB_API LZ4_stream_t* LZ4_initStream (void* buffer, size_t size);


/*! LZ4_streamDecode_t :
 *  Never ever use below internal definitions directly !
 *  These definitions are not API/ABI safe, and may change in future versions.
 *  If you need static allocation, declare or allocate an LZ4_streamDecode_t object.
**/
typedef struct {
    const LZ4_byte* externalDict;
    const LZ4_byte* prefixEnd;
    SIZE_T extDictSize;
    SIZE_T prefixSize;
} LZ4_streamDecode_t_internal;

#define LZ4_STREAMDECODE_MINSIZE 32
union LZ4_streamDecode_u {
    char minStateSize[LZ4_STREAMDECODE_MINSIZE];
    LZ4_streamDecode_t_internal internal_donotuse;
} ;   /* previously typedef'd to LZ4_streamDecode_t */

/*! LZ4_resetStream() :
 *  An LZ4_stream_t structure must be initialized at least once.
 *  This is done with LZ4_initStream(), or LZ4_resetStream().
 *  Consider switching to LZ4_initStream(),
 *  invoking LZ4_resetStream() will trigger deprecation warnings in the future.
 */
LZ4LIB_API void LZ4_resetStream (LZ4_stream_t* streamPtr);


#endif /* LZ4_H_98237428734687 */


#if defined (__cplusplus)
}
#endif
