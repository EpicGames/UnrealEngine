/*
   LZ4 HC - High Compression Mode of LZ4
   Header File
   Copyright (C) 2011-2020, Yann Collet.
   BSD 2-Clause License (http://www.opensource.org/licenses/bsd-license.php)

   Redistribution and use in source and binary forms, with or without
   modification, are permitted provided that the following conditions are
   met:

       * Redistributions of source code must retain the above copyright
   notice, this list of conditions and the following disclaimer.
       * Redistributions in binary form must reproduce the above
   copyright notice, this list of conditions and the following disclaimer
   in the documentation and/or other materials provided with the
   distribution.

   THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS
   "AS IS" AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT
   LIMITED TO, THE IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR
   A PARTICULAR PURPOSE ARE DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT
   OWNER OR CONTRIBUTORS BE LIABLE FOR ANY DIRECT, INDIRECT, INCIDENTAL,
   SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES (INCLUDING, BUT NOT
   LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES; LOSS OF USE,
   DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON ANY
   THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
   (INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE
   OF THIS SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.

   You can contact the author at :
   - LZ4 source repository : https://github.com/lz4/lz4
   - LZ4 public forum : https://groups.google.com/forum/#!forum/lz4c
*/
#ifndef LZ4_HC_H_19834876238432
#define LZ4_HC_H_19834876238432

// HEADER_UNIT_SKIP - Not compatible

/* note : lz4hc requires lz4.h/lz4.c for compilation */
#include "lz4.h"   /* stddef, LZ4LIB_API, LZ4_DEPRECATED */

#if defined (__cplusplus)
extern "C" {
#endif

/* --- Dependency --- */


/* --- Useful constants --- */
#define LZ4HC_CLEVEL_MIN         3
#define LZ4HC_CLEVEL_DEFAULT     9
#define LZ4HC_CLEVEL_OPT_MIN    10
#define LZ4HC_CLEVEL_MAX        12


/*-************************************
 *  Block Compression
 **************************************/
/*! LZ4_compress_HC() :
 *  Compress data from `src` into `dst`, using the powerful but slower "HC" algorithm.
 * `dst` must be already allocated.
 *  Compression is guaranteed to succeed if `dstCapacity >= LZ4_compressBound(srcSize)` (see "lz4.h")
 *  Max supported `srcSize` value is LZ4_MAX_INPUT_SIZE (see "lz4.h")
 * `compressionLevel` : any value between 1 and LZ4HC_CLEVEL_MAX will work.
 *                      Values > LZ4HC_CLEVEL_MAX behave the same as LZ4HC_CLEVEL_MAX.
 * @return : the number of bytes written into 'dst'
 *           or 0 if compression fails.
 */
LZ4LIB_API int LZ4_compress_HC (const char* src, char* dst, int srcSize, int dstCapacity, int compressionLevel);


/* Note :
 *   Decompression functions are provided within "lz4.h" (BSD license)
 */


/*! LZ4_compress_HC_extStateHC() :
 *  Same as LZ4_compress_HC(), but using an externally allocated memory segment for `state`.
 * `state` size is provided by LZ4_sizeofStateHC().
 *  Memory segment must be aligned on 8-bytes boundaries (which a normal malloc() should do properly).
 */
LZ4LIB_API int LZ4_sizeofStateHC(void);
LZ4LIB_API int LZ4_compress_HC_extStateHC(void* stateHC, const char* src, char* dst, int srcSize, int maxDstSize, int compressionLevel);


/*! LZ4_compress_HC_destSize() : v1.9.0+
 *  Will compress as much data as possible from `src`
 *  to fit into `targetDstSize` budget.
 *  Result is provided in 2 parts :
 * @return : the number of bytes written into 'dst' (necessarily <= targetDstSize)
 *           or 0 if compression fails.
 * `srcSizePtr` : on success, *srcSizePtr is updated to indicate how much bytes were read from `src`
 */
LZ4LIB_API int LZ4_compress_HC_destSize(void* stateHC,
                                  const char* src, char* dst,
                                        int* srcSizePtr, int targetDstSize,
                                        int compressionLevel);


/*-************************************
 *  Streaming Compression
 *  Bufferless synchronous API
 **************************************/
 typedef union LZ4_streamHC_u LZ4_streamHC_t;   /* incomplete type (defined later) */

/*! LZ4_createStreamHC() and LZ4_freeStreamHC() :
 *  These functions create and release memory for LZ4 HC streaming state.
 *  Newly created states are automatically initialized.
 *  A same state can be used multiple times consecutively,
 *  starting with LZ4_resetStreamHC_fast() to start a new stream of blocks.
 */
LZ4LIB_API LZ4_streamHC_t* LZ4_createStreamHC(void);
LZ4LIB_API int             LZ4_freeStreamHC (LZ4_streamHC_t* streamHCPtr);

/*
  These functions compress data in successive blocks of any size,
  using previous blocks as dictionary, to improve compression ratio.
  One key assumption is that previous blocks (up to 64 KB) remain read-accessible while compressing next blocks.
  There is an exception for ring buffers, which can be smaller than 64 KB.
  Ring-buffer scenario is automatically detected and handled within LZ4_compress_HC_continue().

  Before starting compression, state must be allocated and properly initialized.
  LZ4_createStreamHC() does both, though compression level is set to LZ4HC_CLEVEL_DEFAULT.

  Selecting the compression level can be done with LZ4_resetStreamHC_fast() (starts a new stream)
  or LZ4_setCompressionLevel() (anytime, between blocks in the same stream) (experimental).
  LZ4_resetStreamHC_fast() only works on states which have been properly initialized at least once,
  which is automatically the case when state is created using LZ4_createStreamHC().

  After reset, a first "fictional block" can be designated as initial dictionary,
  using LZ4_loadDictHC() (Optional).

  Invoke LZ4_compress_HC_continue() to compress each successive block.
  The number of blocks is unlimited.
  Previous input blocks, including initial dictionary when present,
  must remain accessible and unmodified during compression.

  It's allowed to update compression level anytime between blocks,
  using LZ4_setCompressionLevel() (experimental).

  'dst' buffer should be sized to handle worst case scenarios
  (see LZ4_compressBound(), it ensures compression success).
  In case of failure, the API does not guarantee recovery,
  so the state _must_ be reset.
  To ensure compression success
  whenever `dst` buffer size cannot be made >= LZ4_compressBound(),
  consider using LZ4_compress_HC_continue_destSize().

  Whenever previous input blocks can't be preserved unmodified in-place during compression of next blocks,
  it's possible to copy the last blocks into a more stable memory space, using LZ4_saveDictHC().
  Return value of LZ4_saveDictHC() is the size of dictionary effectively saved into 'safeBuffer' (<= 64 KB)

  After completing a streaming compression,
  it's possible to start a new stream of blocks, using the same LZ4_streamHC_t state,
  just by resetting it, using LZ4_resetStreamHC_fast().
*/

LZ4LIB_API void LZ4_resetStreamHC_fast(LZ4_streamHC_t* streamHCPtr, int compressionLevel);   /* v1.9.0+ */
LZ4LIB_API int  LZ4_loadDictHC (LZ4_streamHC_t* streamHCPtr, const char* dictionary, int dictSize);

LZ4LIB_API int LZ4_compress_HC_continue (LZ4_streamHC_t* streamHCPtr,
                                   const char* src, char* dst,
                                         int srcSize, int maxDstSize);

/*! LZ4_compress_HC_continue_destSize() : v1.9.0+
 *  Similar to LZ4_compress_HC_continue(),
 *  but will read as much data as possible from `src`
 *  to fit into `targetDstSize` budget.
 *  Result is provided into 2 parts :
 * @return : the number of bytes written into 'dst' (necessarily <= targetDstSize)
 *           or 0 if compression fails.
 * `srcSizePtr` : on success, *srcSizePtr will be updated to indicate how much bytes were read from `src`.
 *           Note that this function may not consume the entire input.
 */
LZ4LIB_API int LZ4_compress_HC_continue_destSize(LZ4_streamHC_t* LZ4_streamHCPtr,
                                           const char* src, char* dst,
                                                 int* srcSizePtr, int targetDstSize);

LZ4LIB_API int LZ4_saveDictHC (LZ4_streamHC_t* streamHCPtr, char* safeBuffer, int maxDictSize);



/*^**********************************************
 * !!!!!!   STATIC LINKING ONLY   !!!!!!
 ***********************************************/

/*-******************************************************************
 * PRIVATE DEFINITIONS :
 * Do not use these definitions directly.
 * They are merely exposed to allow static allocation of `LZ4_streamHC_t`.
 * Declare an `LZ4_streamHC_t` directly, rather than any type below.
 * Even then, only do so in the context of static linking, as definitions may change between versions.
 ********************************************************************/

#define LZ4HC_DICTIONARY_LOGSIZE 16
#define LZ4HC_MAXD (1<<LZ4HC_DICTIONARY_LOGSIZE)
#define LZ4HC_MAXD_MASK (LZ4HC_MAXD - 1)

#define LZ4HC_HASH_LOG 15
#define LZ4HC_HASHTABLESIZE (1 << LZ4HC_HASH_LOG)
#define LZ4HC_HASH_MASK (LZ4HC_HASHTABLESIZE - 1)


/* Never ever use these definitions directly !
 * Declare or allocate an LZ4_streamHC_t instead.
**/
typedef struct LZ4HC_CCtx_internal LZ4HC_CCtx_internal;
struct LZ4HC_CCtx_internal
{
    LZ4_u32   hashTable[LZ4HC_HASHTABLESIZE];
    LZ4_u16   chainTable[LZ4HC_MAXD];
    const LZ4_byte* end;       /* next block here to continue on current prefix */
    const LZ4_byte* prefixStart;  /* Indexes relative to this position */
    const LZ4_byte* dictStart; /* alternate reference for extDict */
    LZ4_u32   dictLimit;       /* below that point, need extDict */
    LZ4_u32   lowLimit;        /* below that point, no more dict */
    LZ4_u32   nextToUpdate;    /* index from which to continue dictionary update */
    short     compressionLevel;
    LZ4_i8    favorDecSpeed;   /* favor decompression speed if this flag set,
                                  otherwise, favor compression ratio */
    LZ4_i8    dirty;           /* stream has to be fully reset if this flag is set */
    const LZ4HC_CCtx_internal* dictCtx;
};

#define LZ4_STREAMHC_MINSIZE  262200  /* static size, for inter-version compatibility */
union LZ4_streamHC_u {
    char minStateSize[LZ4_STREAMHC_MINSIZE];
    LZ4HC_CCtx_internal internal_donotuse;
}; /* previously typedef'd to LZ4_streamHC_t */

/* LZ4_streamHC_t :
 * This structure allows static allocation of LZ4 HC streaming state.
 * This can be used to allocate statically on stack, or as part of a larger structure.
 *
 * Such state **must** be initialized using LZ4_initStreamHC() before first use.
 *
 * Note that invoking LZ4_initStreamHC() is not required when
 * the state was created using LZ4_createStreamHC() (which is recommended).
 * Using the normal builder, a newly created state is automatically initialized.
 *
 * Static allocation shall only be used in combination with static linking.
 */

/* LZ4_initStreamHC() : v1.9.0+
 * Required before first use of a statically allocated LZ4_streamHC_t.
 * Before v1.9.0 : use LZ4_resetStreamHC() instead
 */
LZ4LIB_API LZ4_streamHC_t* LZ4_initStreamHC(void* buffer, size_t size);

/* LZ4_resetStreamHC() is now replaced by LZ4_initStreamHC().
 * The intention is to emphasize the difference with LZ4_resetStreamHC_fast(),
 * which is now the recommended function to start a new stream of blocks,
 * but cannot be used to initialize a memory segment containing arbitrary garbage data.
 *
 * It is recommended to switch to LZ4_initStreamHC().
 * LZ4_resetStreamHC() will generate deprecation warnings in a future version.
 */
LZ4LIB_API void LZ4_resetStreamHC (LZ4_streamHC_t* streamHCPtr, int compressionLevel);


#if defined (__cplusplus)
}
#endif

#endif /* LZ4_HC_H_19834876238432 */


/*-**************************************************
 * !!!!!     STATIC LINKING ONLY     !!!!!
 * Following definitions are considered experimental.
 * They should not be linked from DLL,
 * as there is no guarantee of API stability yet.
 * Prototypes will be promoted to "stable" status
 * after successful usage in real-life scenarios.
 ***************************************************/
#ifdef LZ4_HC_STATIC_LINKING_ONLY   /* protection macro */
#ifndef LZ4_HC_SLO_098092834
#define LZ4_HC_SLO_098092834

#define LZ4_STATIC_LINKING_ONLY   /* LZ4LIB_STATIC_API */
#include "lz4.h"

#if defined (__cplusplus)
extern "C" {
#endif

/*! LZ4_setCompressionLevel() : v1.8.0+ (experimental)
 *  It's possible to change compression level
 *  between successive invocations of LZ4_compress_HC_continue*()
 *  for dynamic adaptation.
 */
LZ4LIB_STATIC_API void LZ4_setCompressionLevel(
    LZ4_streamHC_t* LZ4_streamHCPtr, int compressionLevel);

/*! LZ4_favorDecompressionSpeed() : v1.8.2+ (experimental)
 *  Opt. Parser will favor decompression speed over compression ratio.
 *  Only applicable to levels >= LZ4HC_CLEVEL_OPT_MIN.
 */
LZ4LIB_STATIC_API void LZ4_favorDecompressionSpeed(
    LZ4_streamHC_t* LZ4_streamHCPtr, int favor);

/*! LZ4_resetStreamHC_fast() : v1.9.0+
 *  When an LZ4_streamHC_t is known to be in a internally coherent state,
 *  it can often be prepared for a new compression with almost no work, only
 *  sometimes falling back to the full, expensive reset that is always required
 *  when the stream is in an indeterminate state (i.e., the reset performed by
 *  LZ4_resetStreamHC()).
 *
 *  LZ4_streamHCs are guaranteed to be in a valid state when:
 *  - returned from LZ4_createStreamHC()
 *  - reset by LZ4_resetStreamHC()
 *  - memset(stream, 0, sizeof(LZ4_streamHC_t))
 *  - the stream was in a valid state and was reset by LZ4_resetStreamHC_fast()
 *  - the stream was in a valid state and was then used in any compression call
 *    that returned success
 *  - the stream was in an indeterminate state and was used in a compression
 *    call that fully reset the state (LZ4_compress_HC_extStateHC()) and that
 *    returned success
 *
 *  Note:
 *  A stream that was last used in a compression call that returned an error
 *  may be passed to this function. However, it will be fully reset, which will
 *  clear any existing history and settings from the context.
 */
LZ4LIB_STATIC_API void LZ4_resetStreamHC_fast(
    LZ4_streamHC_t* LZ4_streamHCPtr, int compressionLevel);

/*! LZ4_compress_HC_extStateHC_fastReset() :
 *  A variant of LZ4_compress_HC_extStateHC().
 *
 *  Using this variant avoids an expensive initialization step. It is only safe
 *  to call if the state buffer is known to be correctly initialized already
 *  (see above comment on LZ4_resetStreamHC_fast() for a definition of
 *  "correctly initialized"). From a high level, the difference is that this
 *  function initializes the provided state with a call to
 *  LZ4_resetStreamHC_fast() while LZ4_compress_HC_extStateHC() starts with a
 *  call to LZ4_resetStreamHC().
 */
LZ4LIB_STATIC_API int LZ4_compress_HC_extStateHC_fastReset (
    void* state,
    const char* src, char* dst,
    int srcSize, int dstCapacity,
    int compressionLevel);

/*! LZ4_attach_HC_dictionary() :
 *  This is an experimental API that allows for the efficient use of a
 *  static dictionary many times.
 *
 *  Rather than re-loading the dictionary buffer into a working context before
 *  each compression, or copying a pre-loaded dictionary's LZ4_streamHC_t into a
 *  working LZ4_streamHC_t, this function introduces a no-copy setup mechanism,
 *  in which the working stream references the dictionary stream in-place.
 *
 *  Several assumptions are made about the state of the dictionary stream.
 *  Currently, only streams which have been prepared by LZ4_loadDictHC() should
 *  be expected to work.
 *
 *  Alternatively, the provided dictionary stream pointer may be NULL, in which
 *  case any existing dictionary stream is unset.
 *
 *  A dictionary should only be attached to a stream without any history (i.e.,
 *  a stream that has just been reset).
 *
 *  The dictionary will remain attached to the working stream only for the
 *  current stream session. Calls to LZ4_resetStreamHC(_fast) will remove the
 *  dictionary context association from the working stream. The dictionary
 *  stream (and source buffer) must remain in-place / accessible / unchanged
 *  through the lifetime of the stream session.
 */
LZ4LIB_STATIC_API void LZ4_attach_HC_dictionary(
          LZ4_streamHC_t *working_stream,
    const LZ4_streamHC_t *dictionary_stream);

#if defined (__cplusplus)
}
#endif

#endif   /* LZ4_HC_SLO_098092834 */
#endif   /* LZ4_HC_STATIC_LINKING_ONLY */
