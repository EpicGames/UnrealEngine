// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

template <typename T, typename... Options>
constexpr bool CAnyOf = (std::is_same_v<T, Options> || ...);
