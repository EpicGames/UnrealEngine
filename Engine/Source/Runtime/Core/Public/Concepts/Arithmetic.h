// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes an arithmetic type.
	 */
	template <typename T>
	concept CArithmetic = std::is_arithmetic_v<T>;
}
