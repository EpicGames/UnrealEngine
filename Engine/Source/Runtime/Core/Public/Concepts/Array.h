// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE::Private::Array
{
	template <typename T>
	concept CArrayImpl = std::is_array_v<T>;
}

namespace UE
{
	/**
	 * Concept which describes an array type.
	 */
	template <typename T>
	concept CArray = UE::Private::Array::CArrayImpl<const volatile T>;
}
