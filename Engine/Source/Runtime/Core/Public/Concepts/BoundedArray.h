// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/Array.h"
#include "Templates/IsArray.h"

namespace UE::Private::BoundedArray
{
	// This helper concept is used by both UE::CBoundedArray and UE::CUnboundedArray to ensure that the compiler
	// knows that bounded array and unbounded array types are mutually exclusive.
	template <typename T>
	concept CBoundedArrayImpl = TIsBoundedArray<T>::Value;
}

namespace UE
{
	/**
	 * Concept which describes a bounded array type.
	 */
	template <typename T>
	concept CBoundedArray = UE::CArray<T> && UE::Private::BoundedArray::CBoundedArrayImpl<const volatile T>;
}
