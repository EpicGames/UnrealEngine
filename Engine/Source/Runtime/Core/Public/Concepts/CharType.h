// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Traits/IsCharType.h"

namespace UE
{
	/**
	 * Concept which describes a character encoding type.
	 */
	template <typename T>
	concept CCharType = TIsCharType_V<T>;
}
