// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/CharType.h"
#include "Traits/IsCharEncodingCompatibleWith.h"

namespace UE
{
	/**
	 * Concept which describes a character encoding type which is binary compatible with a given character encoding type.
	 *
	 * Examples:
	 * UE::CCompatibleCharType<ANSICHAR, ANSICHAR> == true  // Any character type is compatible with itself
	 * UE::CCompatibleCharType<int, int>           == false // ints are not character types
	 * UE::CCompatibleCharType<ANSICHAR, WIDECHAR> == false // ASCII is not binary compatible with wide strings (different sizes)
	 * UE::CCompatibleCharType<ANSICHAR, UTF8CHAR> == true  // ASCII is binary compatible with UTF-8
	 * UE::CCompatibleCharType<UTF8CHAR, ANSICHAR> == false // UTF-8 is not binary compatible with ANSI
	 */
	template <typename T, typename CharType>
	concept CCompatibleCharType = UE::CCharType<T> && TIsCharEncodingCompatibleWith_V<T, CharType>;
}
