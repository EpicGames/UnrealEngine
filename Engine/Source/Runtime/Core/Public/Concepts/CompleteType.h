// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/StaticAssertCompleteType.h"

namespace UE::Private::CompleteType
{
	template <typename T>
	constexpr bool CheckCompleteType()
	{
		UE_STATIC_ASSERT_COMPLETE_TYPE(T, "Expected a complete type");
		return true;
	}
}

namespace UE
{
	/**
	 * Concept which describes a complete type.
	 *
	 * Unlike most concepts, which will evaluate to false if the type doesn't model the concept,
	 * CCompleteType will always evaluate to true, but cause a compile error if the type is incomplete.
	 * The purpose of it is to be included as an additional constraint of other concepts and catch when they're
	 * evaluated with an incomplete type, which could give a wrong result and result in an ill-defined
	 * program.  This is not important for concepts which only operate on fundamental types like UE::CPointer<T>,
	 * or do not depend upon a type's definition like UE::CSameAs<A, B>.
	 */
	template <typename T>
	concept CCompleteType = UE::Private::CompleteType::CheckCompleteType<T>();
}