# Concept Definition Rules

Rules and conventions for writing concepts.

---

## Rule: Concepts should be defined with a `C` prefix and as adjectives or nouns, rather than
## verbs.

### Motivation

Concepts are a unique type of construct in C++, which deserve their own distinguishing prefix.

They are not types, variables or functions, they are always templates (so a `T` prefix is
redundant), they cannot be instantiated, they are distinct from traits, they have unique
behavior in how they compose and resolve, and they can be used in template parameter lists that
those others cannot.

They should be named as adjectives or nouns because they should be seen as a type category
descriptor, rather than a getter or trait to be invoked, for example `template <CSmartPtr T>`
should be read as "`T` is a smart pointer", whereas `template <CIsSmartPtr T>` is clunky in
this respect (we don't write `template <is_typename T>`).

Additionally, the prefix ensures that we can define concepts which describe existing type
categories with the same name, e.g. `CUObject` is a concept describing a `UObject`, which is
already a class type.

## Rule: New concepts should be placed in the UE namespace.

### Motivation

Before C++20, concepts were defined in a way that would work with `TModels` and `TModels_V`,
with the `C` prefix.  This meant that a number of useful concept names had already been taken
and we wanted to keep the `C` prefix.  So all new C++20 Core concepts should go in `UE::` to
distinguish them from the old form of concept structs.

## Rule: cv-transparent concepts must use the private-impl / `const volatile` injection pattern.

### Motivation

A concept that is intended to describe a category of types without regard for cv-qualification
- i.e. `CThing<TThing<X>>` == `CThing<const TThing<X>>` - must be written carefully so that it
honors concept subsumption.

Subsumption is the rule by which the compiler determines that one set of constraints is
strictly more restrictive than another.  When two overloads differ, partial ordering requires
the compiler to prove that an extra-constrained overload subsumes a less-constrained one.
Getting this wrong produces ambiguous-call errors at call sites when it should be unambiguous.

### Incorrect implementations

**Incorrect — `remove_cv_t` inside the trait:**

```cpp
// Private
template <typename T> inline constexpr bool TIsThing_V            = false;
template <typename T> inline constexpr bool TIsThing_V<TThing<T>> = true;

// Public
template <typename T>
concept CThing = TIsThing_V<std::remove_cv_t<T>>;
```

The compiler has no nested concepts to expand, so `CThing<X>` and `CThing<const X>` do not
subsume each other.

**Incorrect — `const volatile` inside the trait:**

```cpp
// Private
template <typename T> inline constexpr bool TIsThing_V                           = false;
template <typename T> inline constexpr bool TIsThing_V<const volatile TThing<T>> = true;

// Public
template <typename T>
concept CThing = TIsThing_V<const volatile T>;
```

The compiler still has no nested concepts to expand, so `CThing<X>` and `CThing<const X>`
are still treated as non-equivalent, as they would have expanded out to
`TIsThing_V<const volatile X>` and `TIsThing_V<const volatile const X>`, which are identical.

**Incorrect — `remove_cv_t` inside a helper concept:**

```cpp
// Private
template <typename T> inline constexpr bool TIsThing_V            = false;
template <typename T> inline constexpr bool TIsThing_V<TThing<T>> = true;

template <typename T>
concept CThingPrivate = TIsThing_V<T>;

// Public
template <typename T>
concept CThing = CThingPrivate<std::remove_cv_t<T>>;
```

While the compiler can expand the nested `CThingPrivate`, it can't expand the trait and
reason that `std::remove_cv_t<X>` is the same as `std::remove_cv_t<const X>`.  So when
comparing `CThing<X>` and `CThing<const X>`, they expand to
`CThingPrivate<std::remove_cv_t<X>>` and `CThingPrivate<std::remove_cv_t<const X>>`, which
are treated as different.

### Correct implementation

```cpp
// Private
template <typename T> inline constexpr bool TIsThing_V                           = false;
template <typename T> inline constexpr bool TIsThing_V<const volatile TThing<T>> = true;

template <typename T>
concept CThingPrivate = TIsThing_V<T>;

// Public
template <typename T>
concept CThing = CThingPrivate<const volatile T>;
```

This is the correct result:

- `CThing<T>` normalizes to `CThingPrivate<const volatile T>`.
- `CThing<const T>` normalizes to `CThingPrivate<const volatile const T>`, which is collapsed
to `CThingPrivate<const volatile T>` because the compiler knows that a `const const` is `const`.

Now both sides normalize to the same atomic constraint with an identical argument, and the
compiler can prove that every constraint in `CThing<const T>` appears in `CThing<T>`, so
subsumption works, and overload resolution works as expected without ambiguity.

See [this online example](https://godbolt.org/z/njErevrob), or Concepts/TSoftClassPtr.h as a
canonical example of this pattern in UE.
