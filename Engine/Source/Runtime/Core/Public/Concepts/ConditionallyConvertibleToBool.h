// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/CompleteType.h"
#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes a type which is conditionally convertible to bool.
	 */
	template <typename T>
	concept CConditionallyConvertibleToBool =
		UE::CCompleteType<std::remove_reference_t<T>> &&
		requires (const T& Val)
		{
			Val ? 1 : 0;
		};
}
