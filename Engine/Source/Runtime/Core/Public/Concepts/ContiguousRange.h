// Copyright Epic Games, Inc. All Rights Reserved. ** 

#pragma once

#include "Concepts/CompleteType.h"
#include "Traits/IsContiguousContainer.h"
#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes a contiguous range of elements.
	 */
	template <typename T>
	concept CContiguousRange =
		!std::is_reference_v<T> && // TIsContiguousContainer wrongly identifies references to containers as containers
		UE::CCompleteType<T> &&
		TIsContiguousContainer_V<T>;

	/**
	 * Concept which describes a contiguous range of elements or a reference to such a range.
	 */
	template <typename T>
	concept CContiguousRangeOrRef = CContiguousRange<std::remove_reference_t<T>>;
}
