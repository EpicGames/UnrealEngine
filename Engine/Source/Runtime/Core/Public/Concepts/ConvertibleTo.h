// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/StaticAssertCompleteType.h"
#include <type_traits>

namespace UE::Private::ConvertibleTo
{
	// Removes all top level qualifiers, pointers and references, e.g. TRemoveAllCVPtrRef<int* const* volatile&>::Type == int
	template <typename T>
	struct TRemoveAllCVPtrRef
	{
		using Type = T;
	};

	template <typename T> struct TRemoveAllCVPtrRef<T*>               : TRemoveAllCVPtrRef<T> {};
	template <typename T> struct TRemoveAllCVPtrRef<T&>               : TRemoveAllCVPtrRef<T> {};
	template <typename T> struct TRemoveAllCVPtrRef<T&&>              : TRemoveAllCVPtrRef<T> {};
	template <typename T> struct TRemoveAllCVPtrRef<const          T> : TRemoveAllCVPtrRef<T> {};
	template <typename T> struct TRemoveAllCVPtrRef<      volatile T> : TRemoveAllCVPtrRef<T> {};
	template <typename T> struct TRemoveAllCVPtrRef<const volatile T> : TRemoveAllCVPtrRef<T> {};

	template <typename From, typename To>
	constexpr bool IsConvertibleToComplete()
	{
		// If a UE_STATIC_ASSERT_COMPLETE_TYPE fires here on a conversion that should be possible to determine without needing full types,
		// the function IsConvertibleTo() below should be updated to handle that case.
		UE_STATIC_ASSERT_COMPLETE_TYPE(typename TRemoveAllCVPtrRef<From>::Type, "Trying to test convertibility against a type that's incomplete");
		UE_STATIC_ASSERT_COMPLETE_TYPE(typename TRemoveAllCVPtrRef<To  >::Type, "Trying to test convertibility against a type that's incomplete");

		return std::is_convertible_v<From, To>;
	}

	// This exists to try and catch incomplete types in the same way as UE::CCompleteType, but when it
	// comes to conversions, a complete type checker is too blunt - there are many common conversion
	// tests that can be accurately done without needing complete types, and we need to try to handle those.
	template <typename From, typename To>
	constexpr bool IsConvertibleTo()
	{
		// Conversion checks between pointer types are a common case which don't require full types
		if constexpr (std::is_pointer_v<From> && std::is_pointer_v<To>)
		{
			using FromType = std::remove_pointer_t<From>;
			using ToType   = std::remove_pointer_t<To>;

			// Losing cv qualifiers is always a convertibility failure
			if constexpr ((std::is_const_v<FromType> && !std::is_const_v<ToType>) || (std::is_volatile_v<FromType> && !std::is_volatile_v<ToType>))
			{
				return false;
			}
			// Adding qualifiers to the same type is always possible
			else if constexpr (std::is_same_v<const volatile FromType, const volatile ToType>)
			{
				return true;
			}
			// Conversion to void is always possible unless it's a function pointer
			else if (std::is_void_v<ToType>)
			{
				return !std::is_function_v<FromType>;
			}
			// Fall back to the regular convertibility test, checking for complete types
			else 
			{
				return IsConvertibleToComplete<From, To>();
			}
		}
		// Conversion checks between reference types are also a common case
		else if constexpr (std::is_lvalue_reference_v<From> && std::is_lvalue_reference_v<To>)
		{
			using FromType = std::remove_reference_t<From>;
			using ToType   = std::remove_reference_t<To>;

			// Losing cv qualifiers is always a convertibility failure
			if constexpr ((std::is_const_v<FromType> && !std::is_const_v<ToType>) || (std::is_volatile_v<FromType> && !std::is_volatile_v<ToType>))
			{
				return false;
			}
			// Adding qualifiers to the same type is always possible
			else if constexpr (std::is_same_v<const volatile FromType, const volatile ToType>)
			{
				return true;
			}
			// Fall back to the regular convertibility test, checking for complete types
			else 
			{
				return IsConvertibleToComplete<From, To>();
			}
		}
		// Fall back to the regular convertibility test, checking for complete types
		else
		{
			return IsConvertibleToComplete<From, To>();
		}
	}
}

namespace UE
{

/**
 * Concept which describes convertibility from one type to another.
 *
 * We use this instead of std::convertible_to because <concepts> isn't a well supported header yet.
 *
 * We want to use UE::CCompleteType here, but many convertibility tests can be done without full type
 * definitions, so it's not simple to use it in a way that won't break lots of valid code.
 * UE::Private::ConvertibleTo::IsConvertibleTo is not a complete implementation of what's necessary
 * but handles common cases before doing a complete type check.
 */
template <typename From, typename To>
concept CConvertibleTo = UE::Private::ConvertibleTo::IsConvertibleTo<From, To>() && requires { static_cast<To>(std::declval<From>()); };

} // UE
