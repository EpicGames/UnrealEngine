// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/SameAs.h"

#include <type_traits>

namespace UE
{

/**
 * Concept which is satisfied if and only if T decays to the same type as U.
 */
template <typename T, typename U>
concept CDecaysTo = CSameAs<std::decay_t<T>, U>;

} // UE
