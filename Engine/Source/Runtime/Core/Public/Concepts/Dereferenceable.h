// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/CompleteType.h"
#include "Concepts/LValueReference.h"
#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes a type which is dereferenceable to obtain an lvalue reference to another object.
	 */
	template <typename T>
	concept CDereferenceable =
		UE::CCompleteType<std::remove_reference_t<T>> &&
		requires(const T& Val)
		{
			{ *Val } -> UE::CLValueReference;
		};
}
