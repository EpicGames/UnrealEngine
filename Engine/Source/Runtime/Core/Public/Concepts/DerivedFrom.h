// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/CompleteType.h"
#include <type_traits>

namespace UE
{
	/**
	 * Concept which is satisfied if DerivedType is an unambiguous public base class of BaseType.
	 *
	 * We use this instead of std::derived_from because <concepts> isn't a well supported header yet.
	 */
	template <typename DerivedType, typename BaseType>
	concept CDerivedFrom =
		UE::CCompleteType<DerivedType> &&
		UE::CCompleteType<BaseType> &&
		std::is_base_of_v<BaseType, DerivedType> &&
		std::is_convertible_v<const volatile DerivedType*, const volatile BaseType*>;
}
