// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/CompleteType.h"
#include "Concepts/DerivedFrom.h"

template <typename FuncType, typename ThreadSafetyMode>
class TDynamicDelegate;

namespace UE::Private::DynamicDelegate
{
	template <typename FuncType, typename ThreadSafetyMode>
	void Resolve(const volatile TDynamicDelegate<FuncType, ThreadSafetyMode>*);

	void Resolve(...) = delete;
}

namespace UE
{
	/**
	 * Concept which describes a dynamic delegate.
	 */
	template <typename T>
	concept CDynamicDelegate = UE::CCompleteType<T> &&
		requires (T* Ptr)
		{
			UE::Private::DynamicDelegate::Resolve(Ptr);
		};
}
