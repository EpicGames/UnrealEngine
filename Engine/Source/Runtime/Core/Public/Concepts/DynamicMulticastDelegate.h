// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/CompleteType.h"
#include "Concepts/DerivedFrom.h"

template <typename FuncType, typename ThreadSafetyMode>
class TDynamicMulticastDelegate;

namespace UE::Private::DynamicMulticastDelegate
{
	template <typename FuncType, typename ThreadSafetyMode>
	void Resolve(const volatile TDynamicMulticastDelegate<FuncType, ThreadSafetyMode>*);

	void Resolve(...) = delete;
}

namespace UE
{
	/**
	 * Concept which describes a dynamic multicast delegate.
	 */
	template <typename T>
	concept CDynamicMulticastDelegate = UE::CCompleteType<T> &&
		requires (T* Ptr)
		{
			UE::Private::DynamicMulticastDelegate::Resolve(Ptr);
		};
}
