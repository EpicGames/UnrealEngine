// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes an enum type.
	 */
	template <typename T>
	concept CEnum = std::is_enum_v<T>;
}
