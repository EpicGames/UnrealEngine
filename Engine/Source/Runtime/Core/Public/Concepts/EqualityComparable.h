// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Models.h"

/**
 * Describes a type comparable with another type.
 */
struct CEqualityComparableWith {
	template <typename T, typename U>
	auto Requires(bool& Result, const T& A, const U& B) -> decltype(
		Result = A == B,
		Result = B == A,
		Result = A != B,
		Result = B != A
	);
};

/**
 * Describes a type comparable with itself.
 */
struct CEqualityComparable {
	template <typename T>
	auto Requires() -> decltype(
		Refines<CEqualityComparableWith, T, T>()
	);
};
