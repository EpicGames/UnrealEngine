// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes a floating point type.  We use this instead of std::floating_point because <concepts> isn't a well supported header yet.
	 */
	template <typename T>
	concept CFloatingPoint = std::is_floating_point_v<T>;
}
