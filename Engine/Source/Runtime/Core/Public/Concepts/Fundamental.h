// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes a fundamental type.
	 */
	template <typename T>
	concept CFundamental = std::is_fundamental_v<T>;
}
