// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/TypeHash.h"

/**
 * Describes a type with a GetTypeHash overload.
 */
struct CGetTypeHashable {
	template <typename T>
	auto Requires(uint32& Result, const T& Val) -> decltype(
		Result = GetTypeHash(Val)
	);
};
