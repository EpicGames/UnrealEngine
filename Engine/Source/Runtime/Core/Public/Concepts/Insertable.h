// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Describes an insertion operation for a destination type where an instance of another type can be inserted via operator<<.
 */
template <typename DestType>
struct CInsertable {
	template <typename T>
	auto Requires(DestType Dest, T& Val) -> decltype(
		Dest << Val
	);
};

namespace UE
{
	/**
	 * Describes inserting an instance of a value type into a destination type using operator<<.
	 *
	 * Equivalent to CInsertableInto<ValueType, DestType>.
	 * Read as "DestType is insertable from ValueType".
	 * Constrain the destination type with <CInsertable<ValueType> DestType>.
	 */
	template <typename DestType, typename ValueType>
	concept CInsertable = requires(DestType& Dest, ValueType& Val)
	{
		Dest << Val;
	};

	/**
	 * Describes inserting an instance of a value type into a destination type using operator<<.
	 *
	 * Equivalent to CInsertable<DestType, ValueType>.
	 * Read as "ValueType is insertable into DestType".
	 * Constrain the value type with <CInsertableInto<DestType> ValueType>.
	 */
	template <typename ValueType, typename DestType>
	concept CInsertableInto = CInsertable<DestType, ValueType>;
}
