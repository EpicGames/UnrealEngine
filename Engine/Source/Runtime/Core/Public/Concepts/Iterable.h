// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <iterator>

namespace UE::Core::Iterable::Private
{
	using std::begin;
	using std::end;

	template <typename T>
	auto Begin(T&& Range) -> decltype(begin((T&&)Range));

	template <typename T>
	auto End(T&& Range) -> decltype(end((T&&)Range));
}

/**
 * Describes a type that can be iterated over using range-based for loops or standard algorithms.
 * The type must provide valid `begin` and `end` methods, either through member functions or
 * through overloads of `begin` and `end` found via argument-dependent lookup (ADL).
 */
struct CIterable
{
	template <typename T>
	auto Requires(T&& Val) -> decltype(
		UE::Core::Iterable::Private::Begin((T&&)Val),
		UE::Core::Iterable::Private::End((T&&)Val)
	);
};
