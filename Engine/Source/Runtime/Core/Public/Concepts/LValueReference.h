// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes an lvalue reference type.
	 */
	template <typename T>
	concept CLValueReference = std::is_lvalue_reference_v<T>;
}
