// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/SameAs.h"
#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes a type which is not a possibly-cv-qualified or reference to a given type.
	 *
	 * This is useful for templated constructors to prevent them being instantiated for the class type itself
	 * so that it will call a move or copy constructor instead:
	 *
	 * struct FSomeClass
	 * {
	 *     template <CNotCVRefTo<FSomeClass> ArgType>
	 *     FSomeClass(ArgType&& Arg)
	 *     {
	 *     }
	 * };
	 */
	template <typename T, typename SelfType>
	concept CNotCVRefTo = !UE::CSameAs<std::remove_cv_t<std::remove_reference_t<T>>, SelfType>;
}
