// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes a pointer type.
	 */
	template <typename T>
	concept CPointer = std::is_pointer_v<T>;
}
