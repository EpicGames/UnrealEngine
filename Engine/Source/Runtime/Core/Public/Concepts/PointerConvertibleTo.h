// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConvertibleTo.h"

namespace UE
{

/**
 * Concept which describes convertibility of pointers from one subject type to another (i.e. safe upcast).
 */
template <typename From, typename To>
concept CPointerConvertibleTo = UE::CConvertibleTo<From*, To*>;

} // UE
