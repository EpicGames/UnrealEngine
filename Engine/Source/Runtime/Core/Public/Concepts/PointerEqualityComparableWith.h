// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/CompleteType.h"
#include "Concepts/SameAs.h"
#include "Concepts/Void.h"
#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes two types which can be compared for equality as pointers.
	 */
	template <typename T, typename U>
	concept CPointerEqualityComparableWith =
		UE::CSameAs<std::remove_cv_t<T>, std::remove_cv_t<U>> ||
		UE::CVoid<T> ||
		UE::CVoid<U> ||
		(
			UE::CCompleteType<T> &&
			UE::CCompleteType<U> &&
			requires(T* Lhs, U* Rhs)
			{
				Lhs == Rhs;
			}
		);
}
