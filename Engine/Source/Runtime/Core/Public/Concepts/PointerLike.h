// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/ConditionallyConvertibleToBool.h"
#include "Concepts/Dereferenceable.h"

namespace UE
{
	/**
	 * Concept which describes a type which acts like a pointer - notably is dereferenceable and is bool-testable for null.
	 */
	template <typename T>
	concept CPointerLike = UE::CConditionallyConvertibleToBool<T> && UE::CDereferenceable<T>;
}
