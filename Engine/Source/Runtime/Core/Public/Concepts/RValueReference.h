// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes an rvalue reference type.
	 */
	template <typename T>
	concept CRValueReference = std::is_rvalue_reference_v<T>;
}
