// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes a reference type - either lvalue or rvalue.
	 */
	template <typename T>
	concept CReference = std::is_reference_v<T>;
}
