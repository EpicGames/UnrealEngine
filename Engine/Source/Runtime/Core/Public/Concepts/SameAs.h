// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE::Private
{
	// This helper concept is to allow CSameAs to be recognized as being a commutative relation, which is necessary for correct
	// partial ordering.
	//
	// For example:
	//
	// template <typename X, typename Y>
	//    requires UE::CSameAs<X, Y>
	// void Func();
	//
	// template <typename X, typename Y>
	//    requires UE::CSameAs<Y, X> && UE::CFloatingPoint<X>
	// void Func();
	//
	// void Test()
	// {
	//     Func<float, float>(); // should call second overload
	// }
	//
	// When performing partial ordering with this commutativity helper, the compiler will expand out the concepts like this:
	//
	// template <typename X, typename Y>
	//    requires UE::Private::CSameAsHelper<X, Y> && UE::Private::CSameAsHelper<Y, X>
	// void Func();
	//
	// template <typename X, typename Y>
	//    requires UE::Private::CSameAsHelper<Y, X> && UE::Private::CSameAsHelper<X, Y> && UE::CFloatingPoint<X>
	// void Func();
	//
	// The compiler can tell the second overload has a strict subset of the constraints of the first overload (concept order
	// doesn't matter), and will thus be chosen as a better match due to partial ordering when X is floating point.
	//
	// Without the helpers, the compiler sees no relationship between UE::Private::CSameAs<X, Y> and UE::Private::CSameAs<Y, X>
	// and so the call to Func<float, float>() would be ambiguous and a compile error.
	template <typename T, typename U>
	concept CSameAsHelper = std::is_same_v<T, U>;
}

namespace UE
{
	/**
	 * Concept which is satisfied if and only if T and U are the same type.
	 *
	 * We use this instead of std::same_as because <concepts> isn't a well supported header yet.
	 */
	template <typename T, typename U>
	concept CSameAs = UE::Private::CSameAsHelper<T, U> && UE::Private::CSameAsHelper<U, T>;
}
