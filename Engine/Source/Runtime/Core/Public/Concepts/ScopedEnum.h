// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/ConvertibleTo.h"
#include "Concepts/Enum.h"

namespace UE
{
	/**
	 * Concept which describes a scoped enum type (aka. enum class).
	 */
	template <typename T>
	concept CScopedEnum = UE::CEnum<T> && !CConvertibleTo<T, int>;
}
