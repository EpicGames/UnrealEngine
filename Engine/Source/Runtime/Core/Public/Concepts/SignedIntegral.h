// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes a signed integral type.  We use this instead of std::signed_integral because <concepts> isn't a well supported header yet.
	 */
	template <typename T>
	concept CSignedIntegral = std::is_integral_v<T> && std::is_signed_v<T>;
}
