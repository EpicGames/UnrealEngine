// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/CompleteType.h"
#include "Concepts/DerivedFrom.h"

template <typename MulticastDelegate, typename OwningClass, typename DelegateInfoClass>
struct TSparseDynamicDelegate;

namespace UE::Private::SparseDynamicMulticastDelegate
{
	template <typename MulticastDelegate, typename OwningClass, typename DelegateInfoClass>
	void Resolve(const volatile TSparseDynamicDelegate<MulticastDelegate, OwningClass, DelegateInfoClass>*);

	void Resolve(...) = delete;
}

namespace UE
{
	/**
	 * Concept which describes a sparse dynamic multicast delegate.
	 */
	template <typename T>
	concept CSparseDynamicMulticastDelegate = UE::CCompleteType<T> &&
		requires (T* Ptr)
		{
			UE::Private::SparseDynamicMulticastDelegate::Resolve(Ptr);
		};
}
