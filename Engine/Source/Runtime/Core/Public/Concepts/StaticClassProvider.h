// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UClass;

/**
 * Describes a type with a StaticClass (static) member.
 */
struct CStaticClassProvider
{
	template <typename T>
	auto Requires(UClass*& ClassRef) -> decltype(
		ClassRef = T::StaticClass()
	);
};
