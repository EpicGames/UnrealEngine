// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UScriptStruct;

/**
 * Describes a type with a StaticStruct (static) member.
 */
struct CStaticStructProvider
{
	template <typename T>
	auto Requires(UScriptStruct*& StructRef) -> decltype(
		StructRef = T::StaticStruct()
	);
};
