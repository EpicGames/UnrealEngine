// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/CharType.h"
#include "Concepts/ContiguousRange.h"
#include "Traits/ElementType.h"
#include <type_traits>

namespace UE
{
	// Concept which describes a type which can be bound to a TStringView.
	template <typename CharRangeType>
	concept CStringViewable =
		(
			std::is_pointer_v<std::decay_t<CharRangeType>> &&
			UE::CCharType<std::remove_pointer_t<std::decay_t<CharRangeType>>>
		) || (
			UE::CContiguousRange<CharRangeType> &&
			UE::CCharType<TElementType_T<CharRangeType>>
		);
}
