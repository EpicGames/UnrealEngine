// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/SameAs.h"

template <typename T>
class TSoftClassPtr;

namespace UE::Private::SoftClassPtr
{
	template <typename T>
	inline constexpr bool TIsTSoftClassPtr_V = false;

	template <typename T>
	inline constexpr bool TIsTSoftClassPtr_V<const volatile TSoftClassPtr<T>> = true;

	template <typename T>
	concept CTSoftClassPtrImpl = TIsTSoftClassPtr_V<T>;
}

namespace UE
{
	/**
	 * Concept which describes a TSoftClassPtr.
	 */
	template <typename T>
	concept CTSoftClassPtr = UE::Private::SoftClassPtr::CTSoftClassPtrImpl<const volatile T>;
}
