// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/CompleteType.h"
#include "Concepts/Enum.h"
#include "Concepts/SameAs.h"
#include "UObject/CoreReflectedTypeAccessors.h"

#include <type_traits>

namespace UE
{
	/**
	* Concept which describes a UEnum (i.e. an enum marked up with UENUM(), not a UObject derived from UEnum).
	*/
	template <typename T>
	concept CUEnum =
		UE::CEnum<T> &&
		//  We try to guard against incomplete types here, but this will succeed for forward declared enums with an underlying type, so
		// it doesn't really work.  Until a solution is found, it's possible for this concept to be memoized in an incorrect state, so
		// care must be taken.
		UE::CCompleteType<T> &&
		requires (T* Ptr)
		{
			{ StaticEnum<std::remove_cv_t<T>>() } -> UE::CSameAs<UEnum*>;
		};
}
