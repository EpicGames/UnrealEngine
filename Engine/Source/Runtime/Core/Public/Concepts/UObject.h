// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/DerivedFrom.h"

class UObject;

namespace UE
{
	/**
	 * Concept which describes a UObject, that is a type derived from UObject.
	 * Pointers are not UObject types - use the following to match pointers to UObjects:
	 *
	 * UE::CPointer<Type> && UE::CSigned<std::remove_pointer_t<Type>>
	 */
	template <typename T>
	concept CUObject = UE::CDerivedFrom<T, UObject>;
}
