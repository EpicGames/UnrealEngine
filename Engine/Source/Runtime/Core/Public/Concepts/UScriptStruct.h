// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/CompleteType.h"
#include "Concepts/SameAs.h"

#include <type_traits>

class UScriptStruct;

template <typename T>
struct TBaseStructure;

namespace UE
{
	/**
	 * Concept which describes a UScriptStruct.
	 */
	template <typename T>
	concept CUScriptStruct = UE::CCompleteType<T> &&
		requires
		{
			{ TBaseStructure<std::remove_cv_t<T>>::Get() } -> UE::CSameAs<UScriptStruct*>;
		};
}
