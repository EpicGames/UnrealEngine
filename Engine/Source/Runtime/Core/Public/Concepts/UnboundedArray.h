// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

#include "Concepts/Array.h"
#include "Concepts/BoundedArray.h"

namespace UE
{
	/**
	 * Concept which describes an unbounded array type.
	 */
	template <typename T>
	concept CUnboundedArray = UE::CArray<T> && !UE::Private::BoundedArray::CBoundedArrayImpl<const volatile T>;
}
