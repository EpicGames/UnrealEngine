// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/ConvertibleTo.h"
#include "Concepts/Enum.h"

namespace UE
{
	/**
	 * Concept which describes an unscoped enum type (aka. non-class enum).
	 */
	template <typename T>
	concept CUnscopedEnum = UE::CEnum<T> && CConvertibleTo<T, int>;
}
