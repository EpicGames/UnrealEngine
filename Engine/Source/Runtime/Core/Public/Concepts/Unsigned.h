// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes an unsigned type.
	 */
	template <typename T>
	concept CUnsigned = std::is_unsigned_v<T>;
}
