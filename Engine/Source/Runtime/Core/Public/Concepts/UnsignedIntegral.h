// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes an unsigned integral type.  We use this instead of std::unsigned_integral because <concepts> isn't a well supported header yet.
	 */
	template <typename T>
	concept CUnsignedIntegral = std::is_integral_v<T> && std::is_unsigned_v<T>;
}
