// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE
{
	/**
	 * Concept which describes a void type.
	 */
	template <typename T>
	concept CVoid = std::is_void_v<T>;
}
