// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Requires.h"
#include <type_traits>

/**
 * TAdderRef and TAdderReserverRef are type-erasing adapters that allow a function to add to (and reserve)
 * a container without the function knowing about the specific type of the container. This allows the 
 * container and allocator to vary without having to change the function signature or make it a template.
 *
 * Example:
 *
 * void AddThree(TAdderRef<int32> Adder)
 * {
 *     Adder.Add(3);
 * }
 *
 * void AddZeroToFour(TAdderReserverRef<int32> AdderReserver)
 * {
 *     AdderReserver.Reserver(AdderReserver.Num() + 5);
 *     for (int Val = 0; Val != 5; ++Val)
 *     {
 *         AdderReserver.Add(Val);
 *     }
 * }
 *
 * TArray<int32, TInlineAllocator<10>> Arr;
 * TSet<int32> Set;
 *
 * AddThree(Arr);      // Arr == { 3 }
 * AddThree(Set);      // Set == { 3 }
 * AddZeroToFour(Arr); // Arr == { 3, 0, 1, 2, 3, 4 }
 * AddZeroToFour(Set); // Set == { 3, 0, 1, 2, 4 } - only contains one 3 because it's a set
 */

namespace UE::Core::Private
{
	template <typename T>
	struct TAdderVTable
	{
		constexpr TAdderVTable(void (*InAddConstRef)(void* ContainerPtr, const T& Val), void (*InAddRValueRef)(void* ContainerPtr, T&& Val))
			: AddConstRef(InAddConstRef)
			, AddRValueRef(InAddRValueRef)
		{
		}

		void (*AddConstRef)(void* ContainerPtr, const T& Val);
		void (*AddRValueRef)(void* ContainerPtr, T&& Val);
	};

	template <typename T, typename SizeType>
	struct TAdderReserverVTable : TAdderVTable<T>
	{
		constexpr TAdderReserverVTable(void (*InAddConstRef)(void* ContainerPtr, const T& Val), void (*InAddRValueRef)(void* ContainerPtr, T&& Val), SizeType (*InNum)(const void* ContainerPtr), void (*InReserve)(void* ContainerPtr, SizeType Size))
			: TAdderVTable<T>(InAddConstRef, InAddRValueRef)
			, Num(InNum)
			, Reserve(InReserve)
		{
		}

		SizeType (*Num)(const void* ContainerPtr);
		void (*Reserve)(void* ContainerPtr, SizeType Size);
	};

	template <typename T, typename ContainerType>
	struct TAdderVTableImpl
	{
		static void AddConstRef(void* ContainerPtr, const T& Val)
		{
			((ContainerType*)ContainerPtr)->Add(Val);
		}

		static void AddRValueRef(void* ContainerPtr, T&& Val)
		{
			((ContainerType*)ContainerPtr)->Add((T&&)Val);
		}
	};

	template <typename SizeType, typename ContainerType>
	struct TAdderReserverVTableImpl
	{
		static SizeType Num(const void* ContainerPtr)
		{
			return ((const ContainerType*)ContainerPtr)->Num();
		}

		static void Reserve(void* ContainerPtr, SizeType Size)
		{
			((ContainerType*)ContainerPtr)->Reserve(Size);
		}
	};

	template <typename T, typename Container>
	constexpr TAdderVTable<T> GAdderVTableImpl(
		&TAdderVTableImpl<T, Container>::AddConstRef,
		&TAdderVTableImpl<T, Container>::AddRValueRef
	);

	template <typename T, typename SizeType, typename Container>
	constexpr TAdderReserverVTable<T, SizeType> GAdderReserverVTableImpl(
		&TAdderVTableImpl<T, Container>::AddConstRef,
		&TAdderVTableImpl<T, Container>::AddRValueRef,
		&TAdderReserverVTableImpl<SizeType, Container>::Num,
		&TAdderReserverVTableImpl<SizeType, Container>::Reserve
	);
}

template <typename T>
struct TAdderRef
{
	template <
		typename ContainerType
		UE_REQUIRES(!std::is_base_of_v<TAdderRef, ContainerType>)
	>
	UE_NODEBUG [[nodiscard]] TAdderRef(ContainerType& InContainer UE_LIFETIMEBOUND)
		: VPtr(&UE::Core::Private::GAdderVTableImpl<T, ContainerType>)
		, ContainerPtr(&InContainer)
	{
	}

	UE_NODEBUG void Add(const T& Val) const
	{
		this->VPtr->AddConstRef(this->ContainerPtr, Val);
	}

	UE_NODEBUG void Add(T&& Val) const
	{
		this->VPtr->AddRValueRef(this->ContainerPtr, (T&&)Val);
	}

protected:
	UE_NODEBUG [[nodiscard]] TAdderRef(const UE::Core::Private::TAdderVTable<T>* InVPtr, void* InContainerPtr)
		: VPtr(InVPtr)
		, ContainerPtr(InContainerPtr)
	{
	}

	const UE::Core::Private::TAdderVTable<T>* VPtr;
	void* ContainerPtr;
};

template <typename T, typename SizeType = int32>
struct TAdderReserverRef : TAdderRef<T>
{
	template <
		typename ContainerType
		UE_REQUIRES(!std::is_base_of_v<TAdderReserverRef, ContainerType>)
	>
	UE_NODEBUG [[nodiscard]] TAdderReserverRef(ContainerType& InContainer UE_LIFETIMEBOUND)
		: TAdderRef<T>(&UE::Core::Private::GAdderReserverVTableImpl<T, SizeType, ContainerType>, &InContainer)
	{
		static_assert(sizeof(InContainer.Num()) <= sizeof(SizeType), "Container has a larger SizeType than the TAdderReserverRef expects");
	}

	UE_NODEBUG [[nodiscard]] SizeType Num() const
	{
		return static_cast<const UE::Core::Private::TAdderReserverVTable<T, SizeType>*>(this->VPtr)->Num(this->ContainerPtr);
	}

	UE_NODEBUG void Reserve(SizeType Size) const
	{
		static_cast<const UE::Core::Private::TAdderReserverVTable<T, SizeType>*>(this->VPtr)->Reserve(this->ContainerPtr, Size);
	}
};
