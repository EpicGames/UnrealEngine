// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "CoreGlobals.h"

/**
 * Fixed-sized allocator that uses a free list to cache allocations.
 * Initial allocation block can be specified in the ctor to warm the cache.
 * Subsequent allocations are in multiples of the given blocksize.
 *
 * Grow can be called at any time to warm the cache with a single block allocation.
 * Initial allocation will want to be a reasonable guess as to how big the pool might grow.
 * BlockSize should be small enough to cut down on allocations but not overcommit memory.
 *
 * NOTE: Currently there is not way to flush the pool because 
 *       it doesn't track each block's allocation status.
 *
 * Not threadsafe <could be implemented using a ThreadingPolicy template parameter>.
 *
 * Template params:
 *   AllocationSize - size of each allocation (must be at least as large as a pointer)
 *   BlockSize - number of allocations to reserve room for when a new block needs to be allocated.
 */
template<uint32 AllocationSize, uint32 BlockSize>
class TAllocatorFixedSizeFreeList
{
public:
	/**
	 * Ctor
	 * 
	 * @param	InitialBlockSize		- number of allocations to warm the cache with
	 */
	TAllocatorFixedSizeFreeList(uint32 InitialBlockSize=0)
		:FreeList(nullptr)
		,NumAllocated(0)
		,NumLive(0)
	{
		// need enough memory to store pointers for the free list
		static_assert(AllocationSize >= sizeof(FreeListNode), "Allocation size must be large enough to hold pointer.");

		// warm the cache
		Grow(InitialBlockSize);
	}

	/**
	 * Destructor. Can't free memory, so only checks that allocations have been returned.
	 */
	~TAllocatorFixedSizeFreeList()
	{
		// by now all block better have been returned to me
		check(GIsCriticalError || NumLive == 0); 
		// WRH - 2007/09/14 - Note we are stranding memory here.
		// These pools are meant to be global and never deleted. 
	}

	/**
	 * Allocates one element from the free list. Return it by calling Free.
	 */
	void *Allocate()
	{
		CheckInvariants();
		if (!FreeList)
		{
			checkSlow(NumLive == NumAllocated);
			Grow(BlockSize);
		}
		// grab next free element, updating FreeList pointer
		void *rawMem = (void *)FreeList;
		FreeList = FreeList->NextFreeAllocation;
		++NumLive;
		CheckInvariants();
		return rawMem;
	}

	/**
	 * Returns one element from the free list.
	 * Must have been acquired previously by Allocate.
	 */
	void Free(void *Element)
	{
		CheckInvariants();
		checkSlow( NumLive > 0 );
		--NumLive;
		FreeListNode* NewFreeElement = (FreeListNode*)Element;
		NewFreeElement->NextFreeAllocation = FreeList;
		FreeList = NewFreeElement;
		CheckInvariants();
	}

	/**
	 * Get total memory allocated.
	 * Only returns the size of allocations made directly by the allocator, not the elements themselves.
	 */
	uint32 GetAllocatedSize() const
	{
		CheckInvariants();
		return NumAllocated * AllocationSize;
	}

	/**
	 * Grows the free list by a specific number of elements. Does one allocation for all elements.
	 * Safe to call at any time to warm the cache with a single block allocation.
	 */
	void Grow(uint32 NumElements)
	{
		if (NumElements == 0) 
		{
			return;
		}

		// need enough memory to store pointers for the free list
		check(AllocationSize*NumElements >= sizeof(FreeListNode));

		// allocate a block of memory
		uint8* RawMem = (uint8*)FMemory::Malloc(AllocationSize * NumElements);
		FreeListNode* NewFreeList = (FreeListNode*)RawMem;
		
		// Chain the block into a list of free list nodes
		for (uint32 i=0;i<NumElements-1;++i,NewFreeList=NewFreeList->NextFreeAllocation)
		{
			NewFreeList->NextFreeAllocation = (FreeListNode*)(RawMem + (i+1)*AllocationSize);
		}

		// link the last Node to the previous FreeList
		NewFreeList->NextFreeAllocation = FreeList;
		FreeList = (FreeListNode*)RawMem;

		NumAllocated += NumElements;
	}

private:

	struct FreeListNode
	{
		FreeListNode* NextFreeAllocation;
	};

	void CheckInvariants() const
	{
		checkSlow(NumAllocated >= NumLive);
	}

private:

	/** Linked List of free memory blocks. */
	FreeListNode* FreeList;

	/** The number of objects that have been allocated. */
	uint32 NumAllocated;

	/** The number of objects that are constructed and "out in the wild". */
	uint32 NumLive;
};
