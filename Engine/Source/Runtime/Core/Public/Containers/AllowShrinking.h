// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainerAllocationPolicies.h"

// Specifies whether or not a container's removal operation should attempt to auto-shrink the container's reserved memory usage
enum class EAllowShrinking : uint8
{
	No,
	Yes,

	Default = Yes /* Prefer UE::Core::Private::AllowShrinkingByDefault<T>() in new code */
};

namespace UE::Core::Private
{
	// Given a container allocation policy, returns EAllowShrinking::Yes or No based on `AllocatorType::ShrinkByDefault`.
	template <typename AllocatorType>
	consteval EAllowShrinking AllowShrinkingByDefault()
	{
		// Convert the ShrinkByDefault enum into an EAllowShrinking.
		// For backwards compatibility, failure to specify `ShrinkByDefault` means Yes.
		return UE::Core::Private::ShrinkByDefaultOr<true, AllocatorType>()
			? EAllowShrinking::Yes
			: EAllowShrinking::No;
	}
}

