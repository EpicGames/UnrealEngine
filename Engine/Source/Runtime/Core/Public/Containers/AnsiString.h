// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Include UnrealString.h.inl's includes before defining the macros, in case the macros 'poison' other headers or there are re-entrant includes.
#include "Containers/UnrealStringIncludes.h.inl" // IWYU pragma: export

#define UE_STRING_CLASS                        FAnsiString
#define UE_STRING_CHARTYPE                     ANSICHAR
#define UE_STRING_CHARTYPE_IS_TCHAR            0
#define UE_STRING_PRINTF_FMT_CHARTYPE          ANSICHAR
#define UE_STRING_DEPRECATED(Version, Message) UE_DEPRECATED(Version, Message)
	#include "Containers/UnrealString.h.inl" // IWYU pragma: export
#undef UE_STRING_DEPRECATED
#undef UE_STRING_PRINTF_FMT_CHARTYPE
#undef UE_STRING_CHARTYPE_IS_TCHAR
#undef UE_STRING_CHARTYPE
#undef UE_STRING_CLASS
