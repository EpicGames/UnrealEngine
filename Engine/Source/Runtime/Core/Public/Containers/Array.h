// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/ReturnFromOpenMode.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/IntrusiveUnsetOptionalState.h"
#include "Misc/ReverseIterate.h"
#include "Misc/UEOps.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/AllowShrinking.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/ContainerElementTypeCompatibility.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryImageWriter.h"

#include "Algo/Heapify.h"
#include "Algo/HeapSort.h"
#include "Algo/IsHeap.h"
#include "Algo/Impl/BinaryHeap.h"
#include "Algo/StableSort.h"
#include "Concepts/GetTypeHashable.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/Less.h"
#include "Templates/LosesQualifiersFromTo.h"
#include "Templates/Requires.h"
#include "Templates/Sorting.h"
#include "Templates/AlignmentTemplates.h"
#include "Traits/ElementType.h"
#include "Traits/IsTriviallyRelocatable.h"

#include <limits>
#include <type_traits>

#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	#define TARRAY_RANGED_FOR_CHECKS 0
#else
	#define TARRAY_RANGED_FOR_CHECKS 1
#endif

template <typename T>
struct TCanBulkSerialize
{
	static constexpr bool Value = std::is_arithmetic_v<T>;
};

// Forward declarations

template <typename T, typename AllocatorType> inline void* operator new(size_t Size, TArray<T, AllocatorType>& Array);
template <typename T, typename AllocatorType> inline void* operator new(size_t Size, TArray<T, AllocatorType>& Array, typename TArray<T, AllocatorType>::SizeType Index);

/**
 * Generic iterator which can operate on types that expose the following:
 * - A type called ElementType representing the contained type.
 * - A method SizeType Num() const that returns the number of items in the container.
 * - A method bool IsValidIndex(SizeType index) which returns whether a given index is valid in the container.
 * - A method T& operator\[\](SizeType index) which returns a reference to a contained object by index.
 * - A method void RemoveAt(SizeType index) which removes the element at index
 */
template <typename ContainerType, typename ElementType, typename SizeType>
class TIndexedContainerIterator
{
	using ContainerAllocatorType = typename ContainerType::AllocatorType;

public:
	UE_NODEBUG [[nodiscard]] TIndexedContainerIterator(ContainerType& InContainer, SizeType StartIndex = 0)
		: Container(InContainer)
		, Index    (StartIndex)
	{
	}

	/** Advances iterator to the next element in the container. */
	UE_NODEBUG TIndexedContainerIterator& operator++()
	{
		++Index;
		return *this;
	}
	UE_NODEBUG TIndexedContainerIterator operator++(int)
	{
		TIndexedContainerIterator Tmp(*this);
		++Index;
		return Tmp;
	}

	/** Moves iterator to the previous element in the container. */
	UE_NODEBUG TIndexedContainerIterator& operator--()
	{
		--Index;
		return *this;
	}
	UE_NODEBUG TIndexedContainerIterator operator--(int)
	{
		TIndexedContainerIterator Tmp(*this);
		--Index;
		return Tmp;
	}

	/** iterator arithmetic support */
	UE_NODEBUG TIndexedContainerIterator& operator+=(SizeType Offset)
	{
		Index += Offset;
		return *this;
	}

	UE_NODEBUG [[nodiscard]] TIndexedContainerIterator operator+(SizeType Offset) const
	{
		TIndexedContainerIterator Tmp(*this);
		return Tmp += Offset;
	}

	UE_NODEBUG TIndexedContainerIterator& operator-=(SizeType Offset)
	{
		return *this += -Offset;
	}

	UE_NODEBUG [[nodiscard]] TIndexedContainerIterator operator-(SizeType Offset) const
	{
		TIndexedContainerIterator Tmp(*this);
		return Tmp -= Offset;
	}

	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT ElementType& operator* () const
	{
		return Container[ Index ];
	}

	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT ElementType* operator->() const
	{
		return &Container[ Index ];
	}

	/** conversion to "bool" returning true if the iterator has not reached the last element. */
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
	{
		return Container.IsValidIndex(Index);
	}

	/** Returns an index to the current element. */
	UE_NODEBUG [[nodiscard]] SizeType GetIndex() const
	{
		return Index;
	}

	/** Resets the iterator to the first element. */
	UE_NODEBUG void Reset()
	{
		Index = 0;
	}

	/** Sets the iterator to one past the last element. */
	UE_NODEBUG void SetToEnd()
	{
		Index = Container.Num();
	}

	/** Removes current element in array. This invalidates the current iterator value and it must be incremented */
	UE_NODEBUG void RemoveCurrent(EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<ContainerAllocatorType>())
	{
		Container.RemoveAt(Index, AllowShrinking);
		Index--;
	}

	/**
	 * Removes current element in array by swapping it with the end element and popping it from the end.
	 * This invalidates the current iterator value and it must be incremented.
	 * Note this modifies the order of the remaining elements in the array.
	 */
	UE_NODEBUG void RemoveCurrentSwap(EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<ContainerAllocatorType>())
	{
		Container.RemoveAtSwap(Index, AllowShrinking);
		Index--;
	}

	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TIndexedContainerIterator& Rhs) const
	{
		return &Container == &Rhs.Container && Index == Rhs.Index;
	}
#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TIndexedContainerIterator& Rhs) const
	{
		return &Container != &Rhs.Container || Index != Rhs.Index;
	}
#endif

private:
	ContainerType& Container;
	SizeType      Index;
};


/** operator + */
template <typename ContainerType, typename ElementType, typename SizeType>
UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT TIndexedContainerIterator<ContainerType, ElementType, SizeType> operator+(SizeType Offset, TIndexedContainerIterator<ContainerType, ElementType, SizeType> RHS)
{
	return RHS + Offset;
}


#if TARRAY_RANGED_FOR_CHECKS
	/**
	 * Pointer-like iterator type for ranged-for loops which checks that the
	 * container hasn't been resized during iteration.
	 */
	template <typename ElementType, typename SizeType, bool bReverse = false>
	struct TCheckedPointerIterator
	{
		// This iterator type only supports the minimal functionality needed to support
		// C++ ranged-for syntax.  For example, it does not provide post-increment ++ or any operation you wouldn't expect from a raw pointer.

		TCheckedPointerIterator() = default;

		UE_NODEBUG [[nodiscard]] explicit TCheckedPointerIterator(const SizeType& InNum, ElementType* InPtr)
			: Ptr       (InPtr)
			, CurrentNum(&InNum)
			, InitialNum(InNum)
		{
		}

		UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT ElementType* operator->() const
		{
			if constexpr (bReverse)
			{
				return Ptr - 1;
			}
			else
			{
				return Ptr;
			}
		}

		UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT ElementType& operator*() const
		{
			if constexpr (bReverse)
			{
				return *(Ptr - 1);
			}
			else
			{
				return *Ptr;
			}
		}

		UE_NODEBUG UE_FORCEINLINE_HINT TCheckedPointerIterator& operator++()
		{
			if constexpr (bReverse)
			{
				--Ptr;
			}
			else
			{
				++Ptr;
			}
			return *this;
		}

		UE_NODEBUG UE_FORCEINLINE_HINT TCheckedPointerIterator operator++(int)
		{
			TCheckedPointerIterator Result = *this;
			++*this;
			return Result;
		}

		UE_NODEBUG UE_FORCEINLINE_HINT TCheckedPointerIterator& operator--()
		{
			if constexpr (bReverse)
			{
				++Ptr;
			}
			else
			{
				--Ptr;
			}
			return *this;
		}

		UE_NODEBUG UE_FORCEINLINE_HINT TCheckedPointerIterator operator--(int)
		{
			TCheckedPointerIterator Result = *this;
			--*this;
			return Result;
		}

		UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT TCheckedPointerIterator operator+(ptrdiff_t N) const
		{
			TCheckedPointerIterator Result = *this;
			if constexpr (bReverse)
			{
				Result.Ptr -= N;
			}
			else
			{
				Result.Ptr += N;
			}
			return Result;
		}

		UE_NODEBUG UE_FORCEINLINE_HINT TCheckedPointerIterator& operator+=(ptrdiff_t N)
		{
			if constexpr (bReverse)
			{
				this->Ptr -= N;
			}
			else
			{
				this->Ptr += N;
			}
			return *this;
		}

		UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT TCheckedPointerIterator operator-(ptrdiff_t N) const
		{
			TCheckedPointerIterator Result = *this;
			if constexpr (bReverse)
			{
				Result.Ptr += N;
			}
			else
			{
				Result.Ptr -= N;
			}
			return Result;
		}

		UE_NODEBUG UE_FORCEINLINE_HINT TCheckedPointerIterator operator-=(ptrdiff_t N)
		{
			if constexpr (bReverse)
			{
				this->Ptr += N;
			}
			else
			{
				this->Ptr -= N;
			}
			return *this;
		}

		UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT ptrdiff_t operator-(const TCheckedPointerIterator& Rhs) const
		{
			if constexpr (bReverse)
			{
				return Rhs.Ptr - Ptr;
			}
			else
			{
				return Ptr - Rhs.Ptr;
			}
		}

		UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT ElementType& operator[](ptrdiff_t N) const
		{
			if constexpr (bReverse)
			{
				return *(Ptr - (N + 1));
			}
			else
			{
				return Ptr[N];
			}
		}

		UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT bool UEOpEquals(const TCheckedPointerIterator& Rhs) const
		{
			// We only need to do the check in comparison operators, because no other operator will be
			// called until after they return.
			//
			// Also, we should only need to check one side of this comparison - comparing iterators from
			// different containers is undefined.

			// This should use *CurrentNum rather than CurrentNum[0], but it works around the following compiler issue:
			// https://developercommunity.visualstudio.com/t/False-C6269-during-static-analysis/10989722
			ensureMsgf(!CurrentNum || CurrentNum[0] == InitialNum, TEXT("Array has changed during ranged-for iteration!"));
			return Ptr == Rhs.Ptr;
		}

		UE_NODEBUG  [[nodiscard]] UE_FORCEINLINE_HINT bool UEOpLessThan(const TCheckedPointerIterator& Rhs) const
		{
			// We only need to do the check in comparison operators, because no other operator will be
			// called until after they return.
			//
			// Also, we should only need to check one side of this comparison - comparing iterators from
			// different containers is undefined.

			// This should use *CurrentNum rather than CurrentNum[0], but it works around the following compiler issue:
			// https://developercommunity.visualstudio.com/t/False-C6269-during-static-analysis/10989722
			ensureMsgf(!CurrentNum || CurrentNum[0] == InitialNum, TEXT("Array has changed during ranged-for iteration!"));
			return Ptr < Rhs.Ptr;
		}

	private:
		ElementType*    Ptr        = nullptr;
		const SizeType* CurrentNum = nullptr;
		SizeType        InitialNum = 0;
	};
#endif


template <typename ElementType, typename IteratorType>
struct TDereferencingIterator
{
	UE_NODEBUG [[nodiscard]] explicit TDereferencingIterator(IteratorType InIter)
		: Iter(InIter)
	{
	}

	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT ElementType& operator*() const
	{
		return *(ElementType*)*Iter;
	}

	UE_NODEBUG UE_FORCEINLINE_HINT TDereferencingIterator& operator++()
	{
		++Iter;
		return *this;
	}

	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TDereferencingIterator& Rhs) const
	{
		return Iter != Rhs.Iter;
	}

private:
	IteratorType Iter;
};

namespace UE::Core::Private
{
	template <typename FromArrayType, typename ToArrayType>
	UE_NODEBUG [[nodiscard]] constexpr bool CanMoveTArrayPointersBetweenArrayTypes()
	{
		using FromAllocatorType          = typename FromArrayType::AllocatorType;
		using ToAllocatorType            = typename ToArrayType::AllocatorType;
		using FromElementType            = typename FromArrayType::ElementType;
		using ToElementType              = typename ToArrayType::ElementType;
		using UnqualifiedFromElementType = std::remove_cv_t<FromElementType>;
		using UnqualifiedToElementType   = std::remove_cv_t<ToElementType>;

		// Allocators must be equal or move-compatible...
		if constexpr (std::is_same_v<FromAllocatorType, ToAllocatorType> || TCanMoveBetweenAllocators<FromAllocatorType, ToAllocatorType>::Value)
		{
			return
				!TLosesQualifiersFromTo_V<FromElementType, ToElementType> &&
				(
					std::is_same_v           <const ToElementType, const FromElementType> ||               // The element type of the container must be the same, or...
					::TIsBitwiseConstructible<UnqualifiedToElementType, UnqualifiedFromElementType>::Value // ... the element type of the source container must be bitwise constructible from the element type in the destination container
				);
		}
		else
		{
			return false;
		}
	}

	// Assume elements are compatible with themselves - avoids problems with generated copy
	// constructors of arrays of forwarded types, e.g.:
	//
	// struct FThing;
	//
	// struct FOuter
	// {
	//     TArray<FThing> Arr; // this will cause errors without this workaround
	// };
	template <typename DestType, typename SourceType>
	constexpr bool TArrayElementsAreCompatible_V = std::disjunction_v<std::is_same<DestType, std::decay_t<SourceType>>, std::is_constructible<DestType, SourceType>>;

	template <typename ElementType, typename AllocatorType>
	UE_NODEBUG static char (&ResolveIsTArrayPtr(const volatile TArray<ElementType, AllocatorType>*))[2];
	UE_NODEBUG static char(&ResolveIsTArrayPtr(...))[1];

	template <typename T>
	constexpr bool TIsTArrayOrDerivedFromTArray_V = sizeof(ResolveIsTArrayPtr((T*)nullptr)) == 2;

	[[noreturn]] CORE_API void OnInvalidArrayNum(unsigned long long NewNum);

	// A hacky way to get the SizeType, since it's defined in the
	// (outer) allocator type, not the (inner) allocator instance type
	template <typename AllocatorInstanceType>
	using TAllocatorSizeType_T = decltype(std::declval<AllocatorInstanceType&>().GetInitialCapacity());

	// Flags are passed as a uint32 to minimize PDB impact of these generated symbols.
	//
	// 1 == TAllocatorTraits<>::SupportsElementAlignment
	// 2 == TAllocatorTraits<>::SupportsSlackTracking
	//
	// When C++20 is guaranteed, concept checks can be used instead.
	template <typename AllocatorType>
	UE_NODEBUG [[nodiscard]] constexpr uint32 GetAllocatorFlags()
	{
		uint32 Result = 0;
		if constexpr (::TAllocatorTraits<AllocatorType>::SupportsElementAlignment)
		{
			Result |= 1;
		}
		if constexpr (::TAllocatorTraits<AllocatorType>::SupportsSlackTracking)
		{
			Result |= 2;
		}
		return Result;
	}

	// Called only when we KNOW we are going to do a realloc increasing by 1.
	// In this case, we know that max == num and can simplify things in a very 
	// hot location in the code.
	// This returns the old ArrayMax in order to save a register clobber/reload.
	template <uint32 Flags, typename AllocatorInstanceType>
	UE_FORCEINLINE_HINT TAllocatorSizeType_T<AllocatorInstanceType> ReallocGrow1_DoAlloc_Impl(
		uint32                                       ElementSize,
		uint32                                       ElementAlignment,
		AllocatorInstanceType&                       AllocatorInstance,
		TAllocatorSizeType_T<AllocatorInstanceType>& ArrayMax
	)
	{		
		using SizeType  = TAllocatorSizeType_T<AllocatorInstanceType>;
		using USizeType = std::make_unsigned_t<SizeType>;

		const USizeType UOldMax = (USizeType)ArrayMax;
		const USizeType UNewNum = UOldMax + 1U;
		const SizeType  OldMax  = (SizeType)UOldMax;
		const SizeType  NewNum  = (SizeType)UNewNum;
		
		// This should only happen when we've underflowed or overflowed SizeType
		if (NewNum < OldMax)
		{
			OnInvalidArrayNum((unsigned long long)UNewNum);
		}

		SizeType NewMax;
		if constexpr (!!(Flags & 1)) // TAllocatorTraits<AllocatorType>::SupportsElementAlignment
		{
			NewMax = AllocatorInstance.CalculateSlackGrow(NewNum, OldMax, ElementSize, ElementAlignment);
			AllocatorInstance.ResizeAllocation(UOldMax, NewMax, ElementSize, ElementAlignment);
		}
		else
		{
			NewMax = AllocatorInstance.CalculateSlackGrow(NewNum, OldMax, ElementSize);
			AllocatorInstance.ResizeAllocation(UOldMax, NewMax, ElementSize);
		}
		ArrayMax = NewMax;
		#if UE_ENABLE_ARRAY_SLACK_TRACKING
		if constexpr (!!(Flags & 2)) // TAllocatorTraits<AllocatorType>::SupportsSlackTracking
		{
			AllocatorInstance.SlackTrackerLogNum(NewNum);
		}
		#endif

		return OldMax;
	}

	// Version for small sizes/alignments. This allows the parameter setup to be a single instruction
	// note the uint16 limitation allows for a single instruction setup on arm.
	template <uint32 Flags, typename AllocatorInstanceType>
	FORCENOINLINE TAllocatorSizeType_T<AllocatorInstanceType> ReallocGrow1_DoAlloc_Tiny(
		uint16                                       ElementSizeAndAlignment,
		AllocatorInstanceType&                       AllocatorInstance,
		TAllocatorSizeType_T<AllocatorInstanceType>& ArrayMax
	)
	{
		return ReallocGrow1_DoAlloc_Impl<Flags, AllocatorInstanceType>(ElementSizeAndAlignment & 0xff, ElementSizeAndAlignment >> 8, AllocatorInstance, ArrayMax);		
	}

	template <uint32 Flags, typename AllocatorInstanceType>
	FORCENOINLINE TAllocatorSizeType_T<AllocatorInstanceType> ReallocGrow1_DoAlloc(
		uint32                                       ElementSize,
		uint32                                       ElementAlignment,
		AllocatorInstanceType&                       AllocatorInstance,
		TAllocatorSizeType_T<AllocatorInstanceType>& ArrayMax
	)
	{
		return ReallocGrow1_DoAlloc_Impl<Flags, AllocatorInstanceType>(ElementSize, ElementAlignment, AllocatorInstance, ArrayMax);		
	}

	// This should be used for repeated growing operations when reallocations are to be amortized over multiple inserts.
	template <uint32 Flags, typename AllocatorInstanceType>
	FORCENOINLINE TAllocatorSizeType_T<AllocatorInstanceType> ReallocGrow(
		uint32                                       ElementSize,
		uint32                                       ElementAlignment,
		TAllocatorSizeType_T<AllocatorInstanceType>  Count,
		AllocatorInstanceType&                       AllocatorInstance,
		TAllocatorSizeType_T<AllocatorInstanceType>& ArrayNum,
		TAllocatorSizeType_T<AllocatorInstanceType>& ArrayMax
	)
	{
		using SizeType  = TAllocatorSizeType_T<AllocatorInstanceType>;
		using USizeType = std::make_unsigned_t<SizeType>;

		const USizeType UCount  = (USizeType)Count;
		const USizeType UOldNum = (USizeType)ArrayNum;
		const USizeType UOldMax = (USizeType)ArrayMax;
		const USizeType UNewNum = UOldNum + UCount;
		const SizeType  OldNum  = (SizeType)UOldNum;
		const SizeType  OldMax  = (SizeType)UOldMax;
		const SizeType  NewNum  = (SizeType)UNewNum;

		checkSlow((OldNum >= 0) & (OldMax >= OldNum) & (Count >= 0)); // & for one branch

		ArrayNum = NewNum;

#if DO_GUARD_SLOW
		if (UNewNum > UOldMax)
#else
		// SECURITY - This check will guard against negative counts too, in case the checkSlow above is compiled out.
		// However, it results in slightly worse code generation.
		if (UCount > UOldMax - UOldNum)
#endif
		{
			// This should only happen when we've underflowed or overflowed SizeType
			if (NewNum < OldNum)
			{
				OnInvalidArrayNum((unsigned long long)UNewNum);
			}
			SizeType NewMax;
			if constexpr (!!(Flags & 1)) // TAllocatorTraits<AllocatorType>::SupportsElementAlignment
			{
				NewMax = AllocatorInstance.CalculateSlackGrow(NewNum, OldMax, ElementSize, ElementAlignment);
				AllocatorInstance.ResizeAllocation(UOldNum, NewMax, ElementSize, ElementAlignment);
			}
			else
			{
				NewMax = AllocatorInstance.CalculateSlackGrow(NewNum, OldMax, ElementSize);
				AllocatorInstance.ResizeAllocation(UOldNum, NewMax, ElementSize);
			}
			ArrayMax = NewMax;
#if UE_ENABLE_ARRAY_SLACK_TRACKING
			if constexpr (!!(Flags & 2)) // TAllocatorTraits<AllocatorType>::SupportsSlackTracking
			{
				AllocatorInstance.SlackTrackerLogNum(NewNum);
			}
#endif
		}

		return OldNum;
	}

	// This should be used for repeated shrinking operations when reallocations are to be amortized over multiple removals.
	template <uint32 Flags, typename AllocatorInstanceType>
	FORCENOINLINE void ReallocShrink(
		uint32                                       ElementSize,
		uint32                                       ElementAlignment,
		AllocatorInstanceType&                       AllocatorInstance,
		TAllocatorSizeType_T<AllocatorInstanceType>  ArrayNum,
		TAllocatorSizeType_T<AllocatorInstanceType>& ArrayMax
	)
	{
		using SizeType  = TAllocatorSizeType_T<AllocatorInstanceType>;

		SizeType OldArrayMax = ArrayMax;

		if constexpr (!!(Flags & 1)) // TAllocatorTraits<AllocatorType>::SupportsElementAlignment
		{
			SizeType NewArrayMax = AllocatorInstance.CalculateSlackShrink(ArrayNum, OldArrayMax, ElementSize, ElementAlignment);
			if (NewArrayMax != OldArrayMax)
			{
				ArrayMax = NewArrayMax;
				AllocatorInstance.ResizeAllocation(ArrayNum, NewArrayMax, ElementSize, ElementAlignment);
			}
		}
		else
		{
			SizeType NewArrayMax = AllocatorInstance.CalculateSlackShrink(ArrayNum, OldArrayMax, ElementSize);
			if (NewArrayMax != OldArrayMax)
			{
				ArrayMax = NewArrayMax;
				AllocatorInstance.ResizeAllocation(ArrayNum, NewArrayMax, ElementSize);
			}
		}
	}

	// This should be used for setting an allocation to a specific size.
	// Precondition: NewMax >= ArrayNum.
	template <uint32 Flags, typename AllocatorInstanceType>
	FORCENOINLINE void ReallocTo(
		uint32                                       ElementSize,
		uint32                                       ElementAlignment,
		TAllocatorSizeType_T<AllocatorInstanceType>  NewMax,
		AllocatorInstanceType&                       AllocatorInstance,
		TAllocatorSizeType_T<AllocatorInstanceType>  ArrayNum,
		TAllocatorSizeType_T<AllocatorInstanceType>& ArrayMax
	)
	{
		if constexpr (!!(Flags & 1)) // TAllocatorTraits<AllocatorType>::SupportsElementAlignment
		{
			if (NewMax)
			{
				NewMax = AllocatorInstance.CalculateSlackReserve(NewMax, ElementSize, ElementAlignment);
			}
			if (NewMax != ArrayMax)
			{
				ArrayMax = NewMax;
				AllocatorInstance.ResizeAllocation(ArrayNum, NewMax, ElementSize, ElementAlignment);
			}
		}
		else
		{
			if (NewMax)
			{
				NewMax = AllocatorInstance.CalculateSlackReserve(NewMax, ElementSize);
			}
			if (NewMax != ArrayMax)
			{
				ArrayMax = NewMax;
				AllocatorInstance.ResizeAllocation(ArrayNum, NewMax, ElementSize);
			}
		}
	}

	template <uint32 Flags, typename AllocatorInstanceType>
	FORCENOINLINE void ReallocForCopy(
		uint32                                       ElementSize,
		uint32                                       ElementAlignment,
		TAllocatorSizeType_T<AllocatorInstanceType>  NewMax,
		TAllocatorSizeType_T<AllocatorInstanceType>  PrevMax,
		AllocatorInstanceType&                       AllocatorInstance,
		TAllocatorSizeType_T<AllocatorInstanceType>  ArrayNum,
		TAllocatorSizeType_T<AllocatorInstanceType>& ArrayMax
	)
	{
		if constexpr (!!(Flags & 1)) // TAllocatorTraits<AllocatorType>::SupportsElementAlignment
		{
			if (NewMax)
			{
				NewMax = AllocatorInstance.CalculateSlackReserve(NewMax, ElementSize, ElementAlignment);
			}
			if (NewMax > PrevMax)
			{
				AllocatorInstance.ResizeAllocation(0, NewMax, ElementSize, ElementAlignment);
			}
			else
			{
				NewMax = PrevMax;
			}
		}
		else
		{
			if (NewMax)
			{
				NewMax = AllocatorInstance.CalculateSlackReserve(NewMax, ElementSize);
			}
			if (NewMax > PrevMax)
			{
				AllocatorInstance.ResizeAllocation(0, NewMax, ElementSize);
			}
			else
			{
				NewMax = PrevMax;
			}
		}
		ArrayMax = NewMax;
	}
} // namespace UE::Core::Private


/**
 * Templated dynamic array
 *
 * A dynamically sized array of typed elements.  Makes the assumption that your elements are trivially relocatable; 
 * i.e. that they can be transparently moved to new memory without a copy constructor.  The main implication 
 * is that pointers to elements in the TArray may be invalidated by adding or removing other elements to the array. 
 * Removal of elements is O(N) and invalidates the indices of subsequent elements.
 *
 * Caution: as noted below some methods are not safe for element types that require constructors.
 *
 **/
template<typename InElementType, typename InAllocatorType>
class TArray
{
	template <typename OtherInElementType, typename OtherAllocator>
	friend class TArray;

public:
	using SizeType      = typename InAllocatorType::SizeType ;
	using ElementType   = InElementType;
	using AllocatorType = InAllocatorType;

private:
	using USizeType = typename std::make_unsigned_t<SizeType>;

public:
	using ElementAllocatorType = std::conditional_t<
		AllocatorType::NeedsElementType,
		typename AllocatorType::template ForElementType<ElementType>,
		typename AllocatorType::ForAnyElementType
	>;

	static_assert(std::is_signed_v<SizeType>, "TArray only supports signed index types");

#if UE_AUTORTFM
	// TArray can be returned from AutoRTFM::Open() via a copy-construction in the closed,
	// so long as the element type also supports copy-construction in the closed.
	static constexpr AutoRTFM::EReturnFromOpenMode AutoRTFMReturnFromOpenMode =
		(AutoRTFM::ReturnFromOpenModeFor<ElementType> == AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed) ?
		AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed : AutoRTFM::EReturnFromOpenMode::Unsupported;
#endif

	/**
	 * Constructor, initializes element number counters.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr TArray()
		: ArrayNum(0)
		, ArrayMax(AllocatorInstance.GetInitialCapacity())
	{
	}

	/**
	 * Constructor to create an empty array, that also initializes the ElementAllocatorType from a moved-in parameter.
	 * The input InAllocatorInstance must be empty; it is only meant to pass parameters to the AllocatorInstance.
	 * 
	 * This constructor and the others that take ElementAllocatorType&& are intended to allow AllocatorTypes that need
	 * to be parameterized, such as one that allocates from a specified FMalloc*.
	 * The input AllocatorInstance must be empty because we currently do not have a way to store the correct ArrayMax
	 * for the Allocator's current allocation; that would require adding an additional InArrayMax argument or extending
	 * the API of AllocatorInstance to be able to report it.
	 */
	[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT explicit TArray(ElementAllocatorType&& InAllocatorInstance)
		: AllocatorInstance(MoveTemp(InAllocatorInstance))
		, ArrayNum(0)
		, ArrayMax(AllocatorInstance.GetInitialCapacity())
	{
		check(!AllocatorInstance.HasAllocation());
	}

	/** Explicit consteval constructor for allocators that require zero-initialization of members for constant initialization */
	[[nodiscard]] explicit consteval TArray(EConstEval)
		: AllocatorInstance(ConstEval)
		, ArrayNum(0)
		, ArrayMax(AllocatorInstance.GetInitialCapacity())
	{
	}

	/**
	 * Constructor from a raw array of elements.
	 *
	 * @param Ptr   A pointer to an array of elements to copy.
	 * @param Count The number of elements to copy from Ptr.
	 * @see Append
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TArray(const ElementType* Ptr, SizeType Count)
	{
		Construct(Ptr, Count);
	}

	/**
	 * Constructor from a raw array of elements, that also initializes the ElementAllocatorType from a moved-in parameter.
	 * The input InAllocatorInstance must be empty; it is only meant to pass parameters to the AllocatorInstance.
	 * @see TArray(ElementAllocatorType&&) for further comments on the InAllocatorInstance.
	 */
	[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT explicit TArray(ElementAllocatorType&& InAllocatorInstance, const ElementType* Ptr, SizeType Count)
		: AllocatorInstance(MoveTemp(InAllocatorInstance))
	{
		check(!AllocatorInstance.HasAllocation());
		Construct(Ptr, Count);
	}
private:
	UE_FORCEINLINE_HINT void Construct(const ElementType* Ptr, SizeType Count)
	{
		if (Count < 0)
		{
			// Cast to USizeType first to prevent sign extension on negative sizes, producing unusually large values.
			UE::Core::Private::OnInvalidArrayNum((unsigned long long)(USizeType)Count);
		}

		check(Ptr != nullptr || Count == 0);

		CopyToEmpty(Ptr, Count, 0);
	}
public:

	template <typename OtherElementType, typename OtherSizeType>
	[[nodiscard]] explicit TArray(const TArrayView<OtherElementType, OtherSizeType>& Other);

	template <typename OtherElementType, typename OtherSizeType>
	[[nodiscard]] UE_NODEBUG explicit TArray(ElementAllocatorType&& InAllocatorInstance, const TArrayView<OtherElementType, OtherSizeType>& Other);

	/**
	 * Initializer list constructor
	 */
	[[nodiscard]] TArray(std::initializer_list<InElementType> InitList)
	{
		CopyToEmpty(InitList.begin(), (SizeType)InitList.size(), 0);
	}

	/**
	 * Constructor from an initializer list, that also initializes the ElementAllocatorType from a moved-in parameter.
	 * The input InAllocatorInstance must be empty; it is only meant to pass parameters to the AllocatorInstance.
	 * @see TArray(ElementAllocatorType&&) for further comments on the InAllocatorInstance.
	 */
	[[nodiscard]] UE_NODEBUG explicit TArray(ElementAllocatorType&& InAllocatorInstance, std::initializer_list<InElementType> InitList)
		: AllocatorInstance(MoveTemp(InAllocatorInstance))
	{
		check(!AllocatorInstance.HasAllocation());
		CopyToEmpty(InitList.begin(), (SizeType)InitList.size(), 0);
	}

	/**
	 * Copy constructor from a TArray with a different AllocatorType. Use the common routine to perform the copy.
	 *
	 * @param Other The source array to copy.
	 */
	template <
		typename OtherElementType,
		typename OtherAllocator
		UE_REQUIRES(UE::Core::Private::TArrayElementsAreCompatible_V<ElementType, const OtherElementType&>)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT explicit TArray(const TArray<OtherElementType, OtherAllocator>& Other)
	{
		CopyToEmpty(Other.GetData(), Other.Num(), 0);
	}

	/**
	 * Copy Constructor from a TArray with a different AllocatorType, that also initializes this TArray's
	 * ElementAllocatorType from a moved-in parameter before doing the copy.
	 * The input InAllocatorInstance must be empty; it is only meant to pass parameters to the AllocatorInstance.
	 * @see TArray(ElementAllocatorType&&) for further comments on the InAllocatorInstance.
	 */
	template <
		typename OtherElementType,
		typename OtherAllocator
		UE_REQUIRES(UE::Core::Private::TArrayElementsAreCompatible_V<ElementType, const OtherElementType&>)
	>
	[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT explicit TArray(ElementAllocatorType&& InAllocatorInstance, const TArray<OtherElementType, OtherAllocator>& Other)
		: AllocatorInstance(MoveTemp(InAllocatorInstance))
	{
		check(!AllocatorInstance.HasAllocation());
		CopyToEmpty(Other.GetData(), Other.Num(), 0);
	}

	/**
	 * Copy constructor from a TArray with the same AllocatorType. Use the common routine to perform the copy.
	 *
	 * @param Other The source array to copy.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TArray(const TArray& Other)
	{
		CopyToEmpty(Other.GetData(), Other.Num(), 0);
	}

	/**
	 * Copy Constructor from a TArray with the same AllocatorType, that also initializes this TArray's
	 * ElementAllocatorType from a moved-in parameter before doing the copy.
	 * The input InAllocatorInstance must be empty; this function is only meant to pass parameters to the AllocatorInstance.
	 * @see TArray(ElementAllocatorType&&) for further comments on the InAllocatorInstance.
	 */
	[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT explicit TArray(ElementAllocatorType&& InAllocatorInstance, const TArray& Other)
		: AllocatorInstance(MoveTemp(InAllocatorInstance))
	{
		check(!AllocatorInstance.HasAllocation());
		CopyToEmpty(Other.GetData(), Other.Num(), 0);
	}

	/**
	 * Copy constructor from TArray with same AllocatorType, with ExtraSlack. Use the common routine to perform the copy.
	 *
	 * @param Other The source array to copy.
	 * @param ExtraSlack Tells how much extra memory should be preallocated
	 *                   at the end of the array in the number of elements.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TArray(const TArray& Other, SizeType ExtraSlack)
	{
		CopyToEmptyWithSlack(Other.GetData(), Other.Num(), 0, ExtraSlack);
	}

	/**
	 * Copy constructor from TArray with same AllocatorType, with ExtraSlack, that also initializes this TArray's
	 * ElementAllocatorType from a moved-in parameter before doing the copy.
	 * The input InAllocatorInstance must be empty; this function is only meant to pass parameters to the AllocatorInstance.
	 * @see TArray(ElementAllocatorType&&) for further comments on the InAllocatorInstance.
	 */
	[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT explicit TArray(ElementAllocatorType&& InAllocatorInstance, const TArray& Other, SizeType ExtraSlack)
		: AllocatorInstance(MoveTemp(InAllocatorInstance))
	{
		check(!AllocatorInstance.HasAllocation());
		CopyToEmptyWithSlack(Other.GetData(), Other.Num(), 0, ExtraSlack);
	}

	/**
	 * Initializer list assignment operator. First deletes all currently contained elements
	 * and then copies from initializer list.
	 *
	 * @param InitList The initializer_list to copy from.
	 */
	TArray& operator=(std::initializer_list<InElementType> InitList)
	{
		DestructItems(GetData(), ArrayNum);
		CopyToEmpty(InitList.begin(), (SizeType)InitList.size(), ArrayMax);
		return *this;
	}

	/**
	 * Assignment operator. First deletes all currently contained elements
	 * and then copies from other array.
	 *
	 * AllocatorType changing version.
	 *
	 * @param Other The source array to assign from.
	 */
	template<typename OtherAllocatorType>
	TArray& operator=(const TArray<ElementType, OtherAllocatorType>& Other)
	{
		DestructItems(GetData(), ArrayNum);
		CopyToEmpty(Other.GetData(), Other.Num(), ArrayMax);
		return *this;
	}

	/**
	 * Assignment operator. First deletes all currently contained elements
	 * and then copies from other array.
	 *
	 * @param Other The source array to assign from.
	 */
	TArray& operator=(const TArray& Other)
	{
		if (this != &Other)
		{
			DestructItems(GetData(), ArrayNum);
			CopyToEmpty(Other.GetData(), Other.Num(), ArrayMax);
		}
		return *this;
	}

	template <typename OtherElementType, typename OtherSizeType>
	TArray& operator=(const TArrayView<OtherElementType, OtherSizeType>& Other);

private:

	UE_FORCEINLINE_HINT void SlackTrackerNumChanged()
	{
#if UE_ENABLE_ARRAY_SLACK_TRACKING
		if constexpr (::TAllocatorTraits<InAllocatorType>::SupportsSlackTracking)
		{
			AllocatorInstance.SlackTrackerLogNum(ArrayNum);
		}
#endif
	}

	/**
	 * Moves or copies array. Depends on the array type traits.
	 *
	 * @param ToArray Array to move into.
	 * @param FromArray Array to move from.
	 * @param PrevMax The previous allocated size.
	 */
	template <typename FromArrayType, typename ToArrayType>
	static UE_FORCEINLINE_HINT void MoveOrCopy(ToArrayType& ToArray, FromArrayType& FromArray, SizeType PrevMax)
	{
		if constexpr (UE::Core::Private::CanMoveTArrayPointersBetweenArrayTypes<FromArrayType, ToArrayType>())
		{
			// Move

			static_assert(std::is_same_v<TArray, ToArrayType>, "MoveOrCopy is expected to be called with the current array type as the destination");

			using FromAllocatorType = typename FromArrayType::AllocatorType;
			using ToAllocatorType   = typename ToArrayType::AllocatorType;

			if constexpr (TCanMoveBetweenAllocators<FromAllocatorType, ToAllocatorType>::Value)
			{
				ToArray.AllocatorInstance.template MoveToEmptyFromOtherAllocator<FromAllocatorType>(FromArray.AllocatorInstance);
			}
			else
			{
				ToArray.AllocatorInstance.MoveToEmpty(FromArray.AllocatorInstance);
			}

			ToArray  .ArrayNum = (SizeType)FromArray.ArrayNum;
			ToArray  .ArrayMax = (SizeType)FromArray.ArrayMax;

			// Ensure the destination container could hold the source range (when the allocator size types shrink)
			if constexpr (sizeof(USizeType) < sizeof(typename FromArrayType::USizeType))
			{
				if (ToArray.ArrayNum != FromArray.ArrayNum || ToArray.ArrayMax != FromArray.ArrayMax)
				{
					// Cast to USizeType first to prevent sign extension on negative sizes, producing unusually large values.
					UE::Core::Private::OnInvalidArrayNum((unsigned long long)(USizeType)ToArray.ArrayNum);
				}
			}

			FromArray.ArrayNum = 0;
			FromArray.ArrayMax = FromArray.AllocatorInstance.GetInitialCapacity();

			FromArray.SlackTrackerNumChanged();
			ToArray.SlackTrackerNumChanged();
		}
		else
		{
			// Copy

			ToArray.CopyToEmpty(FromArray.GetData(), FromArray.Num(), PrevMax);
		}
	}

	/**
	 * Moves or copies array. Depends on the array type traits.
	 *
	 * @param ToArray Array to move into.
	 * @param FromArray Array to move from.
	 * @param PrevMax The previous allocated size.
	 * @param ExtraSlack Tells how much extra memory should be preallocated
	 *                   at the end of the array in the number of elements.
	 */
	template <typename FromArrayType, typename ToArrayType>
	static UE_FORCEINLINE_HINT void MoveOrCopyWithSlack(ToArrayType& ToArray, FromArrayType& FromArray, SizeType PrevMax, SizeType ExtraSlack)
	{
		if constexpr (UE::Core::Private::CanMoveTArrayPointersBetweenArrayTypes<FromArrayType, ToArrayType>())
		{
			// Move

			MoveOrCopy(ToArray, FromArray, PrevMax);

			USizeType LocalArrayNum = (USizeType)ToArray.ArrayNum;
			USizeType NewMax        = (USizeType)LocalArrayNum + (USizeType)ExtraSlack;

			// This should only happen when we've underflowed or overflowed SizeType
			if ((SizeType)NewMax < (SizeType)LocalArrayNum)
			{
				UE::Core::Private::OnInvalidArrayNum((unsigned long long)ExtraSlack);
			}

			ToArray.Reserve(NewMax);
		}
		else
		{
			// Copy

			ToArray.CopyToEmptyWithSlack(FromArray.GetData(), FromArray.Num(), PrevMax, ExtraSlack);
		}
	}

public:
	/**
	 * Move constructor from a TArray with the same AllocatorInstance type.
	 *
	 * @param Other Array to move from.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TArray(TArray&& Other)
	{
		MoveOrCopy(*this, Other, 0);
	}

	/**
	 * Move constructor from same AllocatorInstance type that also initializes this TArray's ElementAllocatorType from
	 * a moved-in parameter before moving the elements (or entire data pointer) from the other TArray's AllocatorInstance.
	 * The input InAllocatorInstance must be empty; this function is only meant to pass parameters to the AllocatorInstance.
	 * @see TArray(ElementAllocatorType&&) for further comments on the InAllocatorInstance.
	 */
	[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT explicit TArray(ElementAllocatorType&& InAllocatorInstance, TArray&& Other)
		: AllocatorInstance(MoveTemp(InAllocatorInstance))
	{
		check(!AllocatorInstance.HasAllocation());
		MoveOrCopy(*this, Other, 0);
	}

	/**
	 * Move constructor from a TArray with a different AllocatorInstance type.
	 *
	 * @param Other Array to move from.
	 */
	template <
		typename OtherElementType,
		typename OtherAllocator
		UE_REQUIRES(UE::Core::Private::TArrayElementsAreCompatible_V<ElementType, OtherElementType&&>)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT explicit TArray(TArray<OtherElementType, OtherAllocator>&& Other)
	{
		MoveOrCopy(*this, Other, 0);
	}

	/**
	 * Move constructor from different AllocatorInstance type that also initializes this TArray's ElementAllocatorType from
	 * a moved-in parameter before moving the elements from the other TArray's AllocatorInstance.
	 * The input InAllocatorInstance must be empty; this function is only meant to pass parameters to the AllocatorInstance.
	 * @see TArray(ElementAllocatorType&&) for further comments on the InAllocatorInstance.
	 */
	template <
		typename OtherElementType,
		typename OtherAllocator
		UE_REQUIRES(UE::Core::Private::TArrayElementsAreCompatible_V<ElementType, OtherElementType&&>)
	>
	[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT explicit TArray(ElementAllocatorType&& InAllocatorInstance, TArray<OtherElementType, OtherAllocator>&& Other)
		: AllocatorInstance(MoveTemp(InAllocatorInstance))
	{
		check(!AllocatorInstance.HasAllocation());
		MoveOrCopy(*this, Other, 0);
	}

	/**
	 * Move constructor from a TArray with the same or different AllocatorInstance type, with ExtraSlack.
	 *
	 * @param Other Array to move from.
	 * @param ExtraSlack Tells how much extra memory should be preallocated
	 *                   at the end of the array in the number of elements.
	 */
	template <
		typename OtherElementType
		UE_REQUIRES(UE::Core::Private::TArrayElementsAreCompatible_V<ElementType, OtherElementType&&>)
	>
	[[nodiscard]] TArray(TArray<OtherElementType, AllocatorType>&& Other, SizeType ExtraSlack)
	{
		MoveOrCopyWithSlack(*this, Other, 0, ExtraSlack);
	}

	/**
	 * Move constructor from same or different AllocatorInstance type, with ExtraSlack, that also initializes this
	 * TArray's ElementAllocatorType from a moved-in parameter before moving the elements (or entire data pointer) from the other
	 * TArray's AllocatorInstance. The input InAllocatorInstance must be empty; this function is only meant to pass parameters
	 * to the AllocatorInstance.
	 * @see TArray(ElementAllocatorType&&) for further comments on the InAllocatorInstance.
	 */
	template <
		typename OtherElementType
		UE_REQUIRES(UE::Core::Private::TArrayElementsAreCompatible_V<ElementType, OtherElementType&&>)
	>
	[[nodiscard]] UE_NODEBUG explicit TArray(ElementAllocatorType&& InAllocatorInstance, TArray<OtherElementType, AllocatorType>&& Other, SizeType ExtraSlack)
		: AllocatorInstance(MoveTemp(InAllocatorInstance))
	{
		check(!AllocatorInstance.HasAllocation());
		MoveOrCopyWithSlack(*this, Other, 0, ExtraSlack);
	}

	/**
	 * Move assignment operator.
	 *
	 * @param Other Array to assign and move from.
	 */
	TArray& operator=(TArray&& Other)
	{
		if (this != &Other)
		{
			DestructItems(GetData(), ArrayNum);
			MoveOrCopy(*this, Other, ArrayMax);
		}
		return *this;
	}

	/** Destructor. */
	~TArray()
	{
		UE_STATIC_ASSERT_WARN(TIsTriviallyRelocatable_V<InElementType>, "TArray can only be used with trivially relocatable types");

		DestructItems(GetData(), ArrayNum);

		// note ArrayNum, ArrayMax and data pointer are not invalidated
		// they are left unchanged and use-after-destruct will see them the same as before destruct
	}

	///////////////////////////////////////////////
	// Start - intrusive TOptional<TArray> state //
	///////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TArray;

	UE_NODEBUG [[nodiscard]] explicit TArray(FIntrusiveUnsetOptionalState Tag)
		: ArrayNum(0)
		, ArrayMax(-1)
	{
		// Use ArrayMax == -1 as our intrusive state so that the destructor still works without change, as it doesn't use ArrayMax.
	}
	UE_NODEBUG [[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState Tag) const
	{
		return ArrayMax == -1;
	}
	/////////////////////////////////////////////
	// End - intrusive TOptional<TArray> state //
	/////////////////////////////////////////////

	/**
	 * Helper function for returning a typed pointer to the first array entry.
	 *
	 * @returns Pointer to first array entry or nullptr if ArrayMax == 0.
	 */
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT ElementType* GetData() UE_LIFETIMEBOUND
	{
		return (ElementType*)AllocatorInstance.GetAllocation();
	}

	/**
	 * Helper function for returning a typed pointer to the first array entry.
	 *
	 * @returns Pointer to first array entry or nullptr if ArrayMax == 0.
	 */
	[[nodiscard]] UE_REWRITE const ElementType* GetData() const UE_LIFETIMEBOUND
	{
		return const_cast<TArray*>(this)->GetData();
	}

	/** 
	 * Helper function returning the size of the inner type.
	 *
	 * @returns Size in bytes of array type.
	 */
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT static constexpr uint32 GetTypeSize()
	{
		return sizeof(ElementType);
	}

	/** 
	 * Helper function to return the amount of memory allocated by this
	 * container.
	 * Only returns the size of allocations made directly by the container, not the elements themselves.
	 *
	 * @returns Number of bytes allocated by this container.
	 */
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT SIZE_T GetAllocatedSize(void) const
	{
		return AllocatorInstance.GetAllocatedSize(ArrayMax, sizeof(ElementType));
	}

	/**
	 * Returns the amount of slack in this array in elements.
	 *
	 * @see Num, Shrink
	 */
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT SizeType GetSlack() const
	{
		return ArrayMax - ArrayNum;
	}

	/**
	 * Checks array invariants: if array size is greater than or equal to zero and less
	 * than or equal to the maximum.
	 */
	UE_NODEBUG UE_FORCEINLINE_HINT void CheckInvariants() const
	{
		checkSlow((ArrayNum >= 0) & (ArrayMax >= ArrayNum)); // & for one branch
	}

	/**
	 * Checks if index is in array range.
	 *
	 * @param Index Index to check.
	 */
	UE_FORCEINLINE_HINT void RangeCheck(SizeType Index) const
	{
		CheckInvariants();

		// Template property, branch will be optimized out
		if constexpr (AllocatorType::RequireRangeCheck)
		{
			checkf((Index >= 0) & (Index < ArrayNum),TEXT("Array index out of bounds: %lld into an array of size %lld"),(long long)Index, (long long)ArrayNum); // & for one branch
		}
	}

	/**
	 * Checks if a range of indices are in the array range.
	 *
	 * @param Index Index of the start of the range to check.
	 * @param Count Number of elements in the range.
	 */
	UE_FORCEINLINE_HINT void RangeCheck(SizeType Index, SizeType Count) const
	{
		CheckInvariants();

		// Template property, branch will be optimized out
		if constexpr (AllocatorType::RequireRangeCheck)
		{
			checkf((Count >= 0) & (Index >= 0) & (Index + Count <= ArrayNum), TEXT("Array range out of bounds: index %lld and length %lld into an array of size %lld"), (long long)Index, (long long)Count, (long long)ArrayNum); // & for one branch
		}
	}

	/**
	 * Tests if index is valid, i.e. greater than or equal to zero, and less than the number of elements in the array.
	 *
	 * @param Index Index to test.
	 * @returns True if index is valid. False otherwise.
	 */
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT bool IsValidIndex(SizeType Index) const
	{
		return Index >= 0 && Index < ArrayNum;
	}

	/**
	 * Returns true if the array is empty and contains no elements. 
	 *
	 * @returns True if the array is empty.
	 * @see Num
	 */
	[[nodiscard]] UE_REWRITE bool IsEmpty() const
	{
		return ArrayNum == 0;
	}

	/**
	 * Returns number of elements in array.
	 *
	 * @returns Number of elements in array.
	 * @see GetSlack
	 */
	[[nodiscard]] UE_REWRITE SizeType Num() const
	{
		return ArrayNum;
	}

	/** @returns Number of bytes used, excluding slack */
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT SIZE_T NumBytes() const
	{
		return static_cast<SIZE_T>(ArrayNum) * sizeof(ElementType);
	}

	/**
	 * Returns maximum number of elements in array before reallocation.
	 *
	 * @returns Maximum number of elements in array before reallocation.
	 * @see GetSlack
	 */
	[[nodiscard]] UE_REWRITE SizeType Max() const
	{
		return ArrayMax;
	}

	/**
	 * Array bracket operator. Returns reference to element at given index.
	 *
	 * @returns Reference to indexed element.
	 */
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT ElementType& operator[](SizeType Index) UE_LIFETIMEBOUND
	{
		RangeCheck(Index);
		return GetData()[Index];
	}

	/**
	 * Array bracket operator. Returns reference to element at given index.
	 *
	 * Const version of the above.
	 *
	 * @returns Reference to indexed element.
	 */
	[[nodiscard]] UE_REWRITE const ElementType& operator[](SizeType Index) const UE_LIFETIMEBOUND
	{
		return (*const_cast<TArray*>(this))[Index];
	}

	/**
	 * Pops element from the array.
	 *
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 * @returns Popped element.
	 */
	ElementType Pop(EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		RangeCheck(0);
		ElementType Result = MoveTempIfPossible(GetData()[ArrayNum - 1]);
		RemoveAtImpl(ArrayNum - 1);
		if (AllowShrinking == EAllowShrinking::Yes)
		{
			UE::Core::Private::ReallocShrink<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(
				sizeof(ElementType),
				alignof(ElementType),
				AllocatorInstance,
				ArrayNum,
				ArrayMax
			);
		}
		return Result;
	}

	/**
	 * Pushes element into the array.
	 *
	 * @param Item Item to push.
	 */
	UE_NODEBUG UE_FORCEINLINE_HINT void Push(ElementType&& Item)
	{
		Add(MoveTempIfPossible(Item));
	}

	/**
	 * Pushes element into the array.
	 *
	 * Const ref version of the above.
	 *
	 * @param Item Item to push.
	 * @see Pop, Top
	 */
	UE_NODEBUG UE_FORCEINLINE_HINT void Push(const ElementType& Item)
	{
		Add(Item);
	}

	/**
	 * Returns the top element, i.e. the last one.
	 *
	 * @returns Reference to the top element.
	 * @see Pop, Push
	 */
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT ElementType& Top() UE_LIFETIMEBOUND
	{
		return Last();
	}
	[[nodiscard]] UE_REWRITE const ElementType& Top() const UE_LIFETIMEBOUND
	{
		return const_cast<TArray*>(this)->Top();
	}

	/**
	 * Returns n-th last element from the array.
	 *
	 * @param IndexFromTheEnd (Optional) Index from the end of array (default = 0).
	 * @returns Reference to n-th last element from the array.
	 */
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT ElementType& Last(SizeType IndexFromTheEnd = 0) UE_LIFETIMEBOUND
	{
		RangeCheck(ArrayNum - IndexFromTheEnd - 1);
		return GetData()[ArrayNum - IndexFromTheEnd - 1];
	}
	[[nodiscard]] UE_REWRITE const ElementType& Last(SizeType IndexFromTheEnd = 0) const UE_LIFETIMEBOUND
	{
		return const_cast<TArray*>(this)->Last(IndexFromTheEnd);
	}

	/**
	 * Shrinks the array's used memory to smallest possible to store elements currently in it.
	 *
	 * @see Slack
	 */
	UE_FORCEINLINE_HINT void Shrink()
	{
		CheckInvariants();
		if (ArrayMax != ArrayNum)
		{
			UE::Core::Private::ReallocTo<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(
				sizeof(ElementType),
				alignof(ElementType),
				ArrayNum,
				AllocatorInstance,
				ArrayNum,
				ArrayMax
			);
		}
	}

	/**
	 * Finds element within the array.
	 *
	 * @param Item Item to look for.
	 * @param Index Will contain the found index.
	 * @returns True if found. False otherwise.
	 * @see FindLast, FindLastByPredicate
	 */
	UE_NODEBUG UE_FORCEINLINE_HINT bool Find(const ElementType& Item, SizeType& Index) const
	{
		Index = this->Find(Item);
		return Index != INDEX_NONE;
	}

	/**
	 * Finds element within the array.
	 *
	 * @param Item Item to look for.
	 * @returns Index of the found element. INDEX_NONE otherwise.
	 * @see FindLast, FindLastByPredicate
	 */
	[[nodiscard]] SizeType Find(const ElementType& Item) const
	{
		const ElementType* RESTRICT Start = GetData();
		for (const ElementType* RESTRICT Data = Start, *RESTRICT DataEnd = Data + ArrayNum; Data != DataEnd; ++Data)
		{
			if (*Data == Item)
			{
				return static_cast<SizeType>(Data - Start);
			}
		}
		return INDEX_NONE;
	}

	/**
	 * Finds element within the array starting from the end.
	 *
	 * @param Item Item to look for.
	 * @param Index Output parameter. Found index.
	 * @returns True if found. False otherwise.
	 * @see Find, FindLastByPredicate
	 */
	UE_NODEBUG UE_FORCEINLINE_HINT bool FindLast(const ElementType& Item, SizeType& Index) const
	{
		Index = this->FindLast(Item);
		return Index != INDEX_NONE;
	}

	/**
	 * Finds element within the array starting from the end.
	 *
	 * @param Item Item to look for.
	 * @returns Index of the found element. INDEX_NONE otherwise.
	 */
	[[nodiscard]] SizeType FindLast(const ElementType& Item) const
	{
		for (const ElementType* RESTRICT Start = GetData(), *RESTRICT Data = Start + ArrayNum; Data != Start; )
		{
			--Data;
			if (*Data == Item)
			{
				return static_cast<SizeType>(Data - Start);
			}
		}
		return INDEX_NONE;
	}

	/**
	 * Searches an initial subrange of the array for the last occurrence of an element which matches the specified predicate.
	 *
	 * @param Pred Predicate taking array element and returns true if element matches search criteria, false otherwise.
	 * @param Count The number of elements from the front of the array through which to search.
	 * @returns Index of the found element. INDEX_NONE otherwise.
	 */
	template <typename Predicate>
	[[nodiscard]] SizeType FindLastByPredicate(Predicate Pred, SizeType Count) const
	{
		check(Count >= 0 && Count <= this->Num());
		for (const ElementType* RESTRICT Start = GetData(), *RESTRICT Data = Start + Count; Data != Start; )
		{
			--Data;
			if (::Invoke(Pred, *Data))
			{
				return static_cast<SizeType>(Data - Start);
			}
		}
		return INDEX_NONE;
	}

	/**
	 * Searches the array for the last occurrence of an element which matches the specified predicate.
	 *
	 * @param Pred Predicate taking array element and returns true if element matches search criteria, false otherwise.
	 * @returns Index of the found element. INDEX_NONE otherwise.
	 */
	template <typename Predicate>
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT SizeType FindLastByPredicate(Predicate Pred) const
	{
		return FindLastByPredicate(Pred, ArrayNum);
	}

	/**
	 * Finds an item by key (assuming the ElementType overloads operator== for
	 * the comparison).
	 *
	 * @param Key The key to search by.
	 * @returns Index to the first matching element, or INDEX_NONE if none is found.
	 */
	template <typename KeyType>
	[[nodiscard]] SizeType IndexOfByKey(const KeyType& Key) const
	{
		const ElementType* RESTRICT Start = GetData();
		for (const ElementType* RESTRICT Data = Start, *RESTRICT DataEnd = Start + ArrayNum; Data != DataEnd; ++Data)
		{
			if (*Data == Key)
			{
				return static_cast<SizeType>(Data - Start);
			}
		}
		return INDEX_NONE;
	}

	/**
	 * Finds an item by predicate.
	 *
	 * @param Pred The predicate to match.
	 * @returns Index to the first matching element, or INDEX_NONE if none is found.
	 */
	template <typename Predicate>
	[[nodiscard]] SizeType IndexOfByPredicate(Predicate Pred) const
	{
		const ElementType* RESTRICT Start = GetData();
		for (const ElementType* RESTRICT Data = Start, *RESTRICT DataEnd = Start + ArrayNum; Data != DataEnd; ++Data)
		{
			if (::Invoke(Pred, *Data))
			{
				return static_cast<SizeType>(Data - Start);
			}
		}
		return INDEX_NONE;
	}

	/**
	 * Finds an item by key (assuming the ElementType overloads operator== for
	 * the comparison). Time Complexity: O(n), starts iteration from the beginning so better performance if Key is in the front
	 *
	 * @param Key The key to search by.
	 * @returns Pointer to the first matching element, or nullptr if none is found.
	 * @see Find
	 */
	template <typename KeyType>
	[[nodiscard]] ElementType* FindByKey(const KeyType& Key)
	{
		for (ElementType* RESTRICT Data = GetData(), *RESTRICT DataEnd = Data + ArrayNum; Data != DataEnd; ++Data)
		{
			if (*Data == Key)
			{
				return Data;
			}
		}

		return nullptr;
	}
	template <typename KeyType>
	[[nodiscard]] UE_REWRITE const ElementType* FindByKey(const KeyType& Key) const
	{
		return const_cast<TArray*>(this)->FindByKey(Key);
	}

	/**
	 * Finds an element which matches a predicate functor.
	 *
	 * @param Pred The functor to apply to each element.
	 * @returns Pointer to the first element for which the predicate returns true, or nullptr if none is found.
	 * @see FilterByPredicate, ContainsByPredicate
	 */
	template <typename Predicate>
	[[nodiscard]] ElementType* FindByPredicate(Predicate Pred)
	{
		for (ElementType* RESTRICT Data = GetData(), *RESTRICT DataEnd = Data + ArrayNum; Data != DataEnd; ++Data)
		{
			if (::Invoke(Pred, *Data))
			{
				return Data;
			}
		}

		return nullptr;
	}
	template <typename Predicate>
	[[nodiscard]] UE_REWRITE const ElementType* FindByPredicate(Predicate Pred) const
	{
		return const_cast<TArray*>(this)->FindByPredicate(Pred);
	}

	/**
	 * Filters the elements in the array based on a predicate functor.
	 *
	 * @param Pred The functor to apply to each element.
	 * @returns TArray with the same type as this object which contains
	 *          the subset of elements for which the functor returns true.
	 * @see FindByPredicate, ContainsByPredicate
	 */
	template <typename Predicate>
	[[nodiscard]] TArray<ElementType> FilterByPredicate(Predicate Pred) const
	{
		TArray<ElementType> FilterResults;
		for (const ElementType* RESTRICT Data = GetData(), *RESTRICT DataEnd = Data + ArrayNum; Data != DataEnd; ++Data)
		{
			if (::Invoke(Pred, *Data))
			{
				FilterResults.Add(*Data);
			}
		}
		return FilterResults;
	}

	/**
	 * Checks if this array contains the element.
	 *
	 * @returns	True if found. False otherwise.
	 * @see ContainsByPredicate, FilterByPredicate, FindByPredicate
	 */
	template <typename ComparisonType>
	[[nodiscard]] bool Contains(const ComparisonType& Item) const
	{
		for (const ElementType* RESTRICT Data = GetData(), *RESTRICT DataEnd = Data + ArrayNum; Data != DataEnd; ++Data)
		{
			if (*Data == Item)
			{
				return true;
			}
		}
		return false;
	}

	/**
	 * Checks if this array contains an element for which the predicate is true.
	 *
	 * @param Predicate to use
	 * @returns	True if found. False otherwise.
	 * @see Contains, Find
	 */
	template <typename Predicate>
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT bool ContainsByPredicate(Predicate Pred) const
	{
		return FindByPredicate(Pred) != nullptr;
	}

	/**
	 * Equality operator.
	 *
	 * @param OtherArray Array to compare.
	 * @returns True if this array is the same as OtherArray. False otherwise.
	 */
	UE_NODEBUG [[nodiscard]] bool operator==(const TArray& OtherArray) const
	{
		SizeType Count = Num();

		return Count == OtherArray.Num() && CompareItems(GetData(), OtherArray.GetData(), Count);
	}

	/**
	 * Inequality operator.
	 *
	 * @param OtherArray Array to compare.
	 * @returns True if this array is NOT the same as OtherArray. False otherwise.
	 */
#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	[[nodiscard]] UE_REWRITE bool operator!=(const TArray& OtherArray) const
	{
		return !(*this == OtherArray);
	}
#endif

	/**
	 * Bulk serialize array as a single memory blob when loading. Uses regular serialization code for saving
	 * and doesn't serialize at all otherwise (e.g. transient, garbage collection, ...).
	 * 
	 * Requirements:
	 *   - T's << operator needs to serialize ALL member variables in the SAME order they are layed out in memory.
	 *   - T's << operator can NOT perform any fixup operations. This limitation can be lifted by manually copying
	 *     the code after the BulkSerialize call.
	 *   - T can NOT contain any member variables requiring constructor calls or pointers
	 *   - sizeof(ElementType) must be equal to the sum of sizes of it's member variables.
	 *        - e.g. use pragma pack (push,1)/ (pop) to ensure alignment
	 *        - match up uint8/ WORDs so everything always end up being properly aligned
	 *   - Code can not rely on serialization of T if neither IsLoading() nor IsSaving() is true.
	 *   - Can only be called platforms that either have the same endianness as the one the content was saved with
	 *     or had the endian conversion occur in a cooking process like e.g. for consoles.
	 *
	 * Notes:
	 *   - it is safe to call BulkSerialize on TTransArrays
	 *
	 * IMPORTANT:
	 *   - This is Overridden in XeD3dResourceArray.h Please make certain changes are propagated accordingly
	 *
	 * @param Ar	FArchive to bulk serialize this TArray to/from
	 */
	void BulkSerialize(FArchive& Ar, bool bForcePerElementSerialization = false)
	{
		constexpr int32 ElementSize = sizeof(ElementType);
		// Serialize element size to detect mismatch across platforms.
		int32 SerializedElementSize = ElementSize;
		Ar << SerializedElementSize;
	
		if (bForcePerElementSerialization
			|| (Ar.IsSaving()			// if we are saving, we always do the ordinary serialize as a way to make sure it matches up with bulk serialization
			&& !Ar.IsCooking()			// but cooking and transacting is performance critical, so we skip that
			&& !Ar.IsTransacting())		
			|| Ar.IsByteSwapping()		// if we are byteswapping, we need to do that per-element
			)
		{
			Ar << *this;
		}
		else
		{
			CountBytes(Ar);
			if (Ar.IsLoading())
			{
				// Basic sanity checking to ensure that sizes match.
				if (!ensure(SerializedElementSize == ElementSize))
				{
					Ar.SetError();
					return;
				}

				// Serialize the number of elements, block allocate the right amount of memory and deserialize
				// the data as a giant memory blob in a single call to Serialize. Please see the function header
				// for detailed documentation on limitations and implications.
				SizeType NewArrayNum = 0;
				Ar << NewArrayNum;
				if (!ensure(NewArrayNum >= 0 && std::numeric_limits<SizeType>::max() / (SizeType)ElementSize >= NewArrayNum))
				{
					Ar.SetError();
					return;
				}
				Empty(NewArrayNum);
				AddUninitialized(NewArrayNum);
				Ar.Serialize(GetData(), (int64)NewArrayNum * (int64)ElementSize);
			}
			else if (Ar.IsSaving())
			{
				SizeType ArrayCount = Num();
				Ar << ArrayCount;
				Ar.Serialize(GetData(), (int64)ArrayCount * (int64)ElementSize);
			}
		}
	}

	/**
	 * Count bytes needed to serialize this array.
	 *
	 * @param Ar Archive to count for.
	 */
	UE_NODEBUG void CountBytes(FArchive& Ar) const
	{
		Ar.CountBytes(ArrayNum*sizeof(ElementType), ArrayMax*sizeof(ElementType));
	}

	/**
	 * Adds a given number of uninitialized elements into the array.
	 *
	 * Caution, AddUninitialized() will create elements without calling
	 * the constructor and this is not appropriate for element types that
	 * require a constructor to function properly.
	 *
	 * @param Count Number of elements to add.
	 * @returns Number of elements in array before addition.
	 */
	UE_FORCEINLINE_HINT SizeType AddUninitialized()
	{
		// Begin sensitive code! Single element additions/insertions get inlined _everywhere_ and thus
		// punch above their weight with respect to exe size, so we pay a lot of attention to every instruction
		// here until we get in to the NOINLINE functions that actually handle the growth.

		// Modulo some register allocation and pointer offsets, inlined sites on x64 for the "Tiny" path should look roughly like:
		// 
		// mov eax, dword ptr [array + 8h] // load arraynum
		// cmp eax, dword ptr [array + 0Ch] // load arraymax
		// jne [to ArrayNum++]
		// mov ecx, immediate representing size/alignment
		// lea r8, [array + 0Ch]] // array max
		// lea rdx, [array] // allocator instance
		// call
		// lea ecx, [rax+1] // ArrayNum++
		// mov dword ptr [array + 8h], ecx // Save ArrayNum
		//

		// Single cmp, which we can assume because we are adding a single element.
		if (ArrayNum == ArrayMax)
		{
			// Both branches here write the return in to ArrayNum. This is because the function call clobbers the registers
			// and if we assign as part of the return into something we need, the compiler doesn't have to reload the data 
			// into the clobbered register.

			// When we can pack size and alignment into a single 16 bit load, we save a parameter setup instruction
			// for the function call. The 16 bit part is load-bearing for arm codegen.
			// PVS believes this is overwrought because technically sizeof() <= 255 implies alignof <= 255. I think this is far more
			// clear and costs nothing.
			if constexpr (sizeof(ElementType) <= 255 && alignof(ElementType) <= 255) // -V590 
			{
				// Note that the realloc functions are templated ONLY on allocator instance so they are not duplicated
				// in the code for every type!
				ArrayNum = UE::Core::Private::ReallocGrow1_DoAlloc_Tiny<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(sizeof(ElementType) | (alignof(ElementType) << 8), AllocatorInstance, ArrayMax);
			}
			else
			{
				ArrayNum = UE::Core::Private::ReallocGrow1_DoAlloc<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(sizeof(ElementType), alignof(ElementType), AllocatorInstance, ArrayMax);
			}
		}
		// End sensitive code!

		SizeType OldArrayNum = ArrayNum;
		ArrayNum++;
		return OldArrayNum;	
	}
	UE_FORCEINLINE_HINT SizeType AddUninitialized(SizeType Count)
	{
		// Should be SetNumUninitialized?
		return UE::Core::Private::ReallocGrow<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(
			sizeof(ElementType),
			alignof(ElementType),
			Count,
			AllocatorInstance,
			ArrayNum,
			ArrayMax
		);
	}

private:
	void InsertUninitializedImpl(SizeType Index)
	{
		// Begin sensitive code! See comments in AddUninitialized() before touching!
		if (ArrayNum == ArrayMax)
		{
			if constexpr (sizeof(ElementType) <= 255 && alignof(ElementType) <= 255) // -V590 
			{
				ArrayNum = UE::Core::Private::ReallocGrow1_DoAlloc_Tiny<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(sizeof(ElementType) | (alignof(ElementType) << 8), AllocatorInstance, ArrayMax);
			}
			else
			{
				ArrayNum = UE::Core::Private::ReallocGrow1_DoAlloc<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(sizeof(ElementType), alignof(ElementType), AllocatorInstance, ArrayMax);
			}
		}
		// End sensitive code!
		SizeType OldNum = ArrayNum;
		ArrayNum++;		
		ElementType* Data = GetData() + Index;
		RelocateConstructItems<ElementType>((void*)(Data + 1), Data, OldNum - Index);
	}
	void InsertUninitializedImpl(SizeType Index, SizeType Count)
	{
		// Should be SetNumUninitialized?
		SizeType OldNum = UE::Core::Private::ReallocGrow<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(
			sizeof(ElementType),
			alignof(ElementType),
			Count,
			AllocatorInstance,
			ArrayNum,
			ArrayMax
		);
		ElementType* Data = GetData() + Index;
		RelocateConstructItems<ElementType>((void*)(Data + Count), Data, OldNum - Index);
	}
	template <
		typename OtherSizeType
		UE_REQUIRES(sizeof(OtherSizeType) > sizeof(SizeType))
	>
	UE_NODEBUG void InsertUninitializedImpl(SizeType Index, OtherSizeType Count)
	{
		checkf((OtherSizeType)(SizeType)Count == Count, TEXT("Invalid number of elements to add to this array type: %lld"), (long long)(SizeType)Count);
		return InsertUninitializedImpl(Index, (SizeType)Count);
	}

public:
	/**
	 * Inserts a given number of uninitialized elements into the array at given
	 * location.
	 *
	 * Caution, InsertUninitialized() will create elements without calling the
	 * constructor and this is not appropriate for element types that require
	 * a constructor to function properly.
	 *
	 * @param Index Tells where to insert the new elements.
	 * @param Count Number of elements to add.
	 * @see Insert, InsertZeroed, InsertDefaulted
	 */
	UE_NODEBUG UE_FORCEINLINE_HINT void InsertUninitialized(SizeType Index)
	{
		InsertUninitializedImpl(Index);
	}
	UE_NODEBUG UE_FORCEINLINE_HINT void InsertUninitialized(SizeType Index, SizeType Count)
	{
		InsertUninitializedImpl(Index, Count);
	}

	/**
	 * Inserts a given number of zeroed elements into the array at given
	 * location.
	 *
	 * Caution, InsertZeroed() will create elements without calling the
	 * constructor and this is not appropriate for element types that require
	 * a constructor to function properly.
	 *
	 * @param Index Tells where to insert the new elements.
	 * @param Count Number of elements to add.
	 * @see Insert, InsertUninitialized, InsertDefaulted
	 */
	void InsertZeroed(SizeType Index)
	{
		InsertUninitializedImpl(Index);
		FMemory::Memzero(GetData() + Index, sizeof(ElementType));
	}
	void InsertZeroed(SizeType Index, SizeType Count)
	{
		InsertUninitializedImpl(Index, Count);
		FMemory::Memzero(GetData() + Index, Count * sizeof(ElementType));
	}

	/**
	 * Inserts a zeroed element into the array at given location.
	 *
	 * Caution, InsertZeroed_GetRef() will create an element without calling the
	 * constructor and this is not appropriate for element types that require
	 * a constructor to function properly.
	 *
	 * @param Index Tells where to insert the new element.
	 * @return A reference to the newly-inserted element.
	 * @see Insert_GetRef, InsertDefaulted_GetRef
	 */
	ElementType& InsertZeroed_GetRef(SizeType Index) UE_LIFETIMEBOUND
	{
		InsertUninitializedImpl(Index, 1);
		ElementType* Ptr = GetData() + Index;
		FMemory::Memzero(Ptr, sizeof(ElementType));
		return *Ptr;
	}

	/**
	 * Inserts a given number of default-constructed elements into the array at a given
	 * location.
	 *
	 * @param Index Tells where to insert the new elements.
	 * @param Count Number of elements to add.
	 * @see Insert, InsertUninitialized, InsertZeroed
	 */
	void InsertDefaulted(SizeType Index)
	{
		InsertUninitializedImpl(Index);
		DefaultConstructItems<ElementType>((void*)(GetData() + Index), 1);
	}
	void InsertDefaulted(SizeType Index, SizeType Count)
	{
		InsertUninitializedImpl(Index, Count);
		DefaultConstructItems<ElementType>((void*)(GetData() + Index), Count);
	}

	/**
	 * Inserts a default-constructed element into the array at a given
	 * location.
	 *
	 * @param Index Tells where to insert the new element.
	 * @return A reference to the newly-inserted element.
	 * @see Insert_GetRef, InsertZeroed_GetRef
	 */
	ElementType& InsertDefaulted_GetRef(SizeType Index) UE_LIFETIMEBOUND
	{
		InsertUninitializedImpl(Index, 1);
		ElementType* Ptr = GetData() + Index;
		DefaultConstructItems<ElementType>((void*)Ptr, 1);
		return *Ptr;
	}

	/**
	 * Inserts given elements into the array at given location.
	 *
	 * @param InitList Array of elements to insert.
	 * @param InIndex Tells where to insert the new elements.
	 * @returns Location at which the item was inserted.
	 */
	SizeType Insert(std::initializer_list<ElementType> InitList, const SizeType InIndex)
	{
		SizeType NumNewElements = (SizeType)InitList.size();

		InsertUninitializedImpl(InIndex, NumNewElements);
		ConstructItems<ElementType>((void*)(GetData() + InIndex), InitList.begin(), NumNewElements);

		return InIndex;
	}

	/**
	 * Inserts given elements into the array at given location.
	 *
	 * @param Items Array of elements to insert.
	 * @param InIndex Tells where to insert the new elements.
	 * @returns Location at which the item was inserted.
	 */
	template <typename OtherAllocator>
	SizeType Insert(const TArray<ElementType, OtherAllocator>& Items, const SizeType InIndex)
	{
		check((const void*)this != (const void*)&Items);

		auto NumNewElements = Items.Num();

		InsertUninitializedImpl(InIndex, NumNewElements);
		ConstructItems<ElementType>((void*)(GetData() + InIndex), Items.GetData(), NumNewElements);

		return InIndex;
	}

	/**
	 * Inserts given elements into the array at given location.
	 *
	 * @param Items Array of elements to insert.
	 * @param InIndex Tells where to insert the new elements.
	 * @returns Location at which the item was inserted.
	 */
	template <typename OtherAllocator>
	SizeType Insert(TArray<ElementType, OtherAllocator>&& Items, const SizeType InIndex)
	{
		check((const void*)this != (const void*)&Items);

		auto NumNewElements = Items.Num();

		InsertUninitializedImpl(InIndex, NumNewElements);
		RelocateConstructItems<ElementType>((void*)(GetData() + InIndex), Items.GetData(), NumNewElements);
		Items.ArrayNum = 0;

		Items.SlackTrackerNumChanged();

		return InIndex;
	}

	/**
	 * Inserts a raw array of elements at a particular index in the TArray.
	 *
	 * @param Ptr A pointer to an array of elements to add.
	 * @param Count The number of elements to insert from Ptr.
	 * @param Index The index to insert the elements at.
	 * @return The index of the first element inserted.
	 * @see Add, Remove
	 */
	SizeType Insert(const ElementType* Ptr, SizeType Count, SizeType Index)
	{
		check(Ptr != nullptr);

		InsertUninitializedImpl(Index, Count);
		ConstructItems<ElementType>((void*)(GetData() + Index), Ptr, Count);

		return Index;
	}

	/**
	 * Checks that the specified address is not part of an element within the
	 * container. Used for implementations to check that reference arguments
	 * aren't going to be invalidated by possible reallocation.
	 *
	 * @param Addr The address to check.
	 * @see Add, Remove
	 */
	UE_FORCEINLINE_HINT void CheckAddress(const ElementType* Addr) const
	{
		checkf(Addr < GetData() || Addr >= (GetData() + ArrayMax), TEXT("Attempting to use a container element (%p) which already comes from the container being modified (%p, ArrayMax: %lld, ArrayNum: %lld, SizeofElement: %zu)!"), Addr, GetData(), (long long)ArrayMax, (long long)ArrayNum, sizeof(ElementType));
	}

	/**
	 * Inserts a given element into the array at given location. Move semantics
	 * version.
	 *
	 * @param Item The element to insert.
	 * @param Index Tells where to insert the new elements.
	 * @returns Location at which the insert was done.
	 * @see Add, Remove
	 */
	SizeType Insert(ElementType&& Item, SizeType Index)
	{
		CheckAddress(&Item);

		// construct a copy in place at Index (this new operator will insert at 
		// Index, then construct that memory with Item)
		InsertUninitializedImpl(Index);
		::new((void*)(GetData() + Index)) ElementType(MoveTempIfPossible(Item));
		return Index;
	}

	/**
	 * Inserts a given element into the array at given location.
	 *
	 * @param Item The element to insert.
	 * @param Index Tells where to insert the new elements.
	 * @returns Location at which the insert was done.
	 * @see Add, Remove
	 */
	SizeType Insert(const ElementType& Item, SizeType Index)
	{
		CheckAddress(&Item);

		// construct a copy in place at Index (this new operator will insert at 
		// Index, then construct that memory with Item)
		InsertUninitializedImpl(Index);
		::new((void*)(GetData() + Index)) ElementType(Item);
		return Index;
	}

	/**
	 * Inserts a given element into the array at given location. Move semantics
	 * version.
	 *
	 * @param Item The element to insert.
	 * @param Index Tells where to insert the new element.
	 * @return A reference to the newly-inserted element.
	 * @see Add, Remove
	 */
	[[nodiscard]] ElementType& Insert_GetRef(ElementType&& Item, SizeType Index) UE_LIFETIMEBOUND
	{
		CheckAddress(&Item);

		// construct a copy in place at Index (this new operator will insert at 
		// Index, then construct that memory with Item)
		InsertUninitializedImpl(Index);
		ElementType* Ptr = GetData() + Index;
		::new((void*)Ptr) ElementType(MoveTempIfPossible(Item));
		return *Ptr;
	}

	/**
	 * Inserts a given element into the array at given location.
	 *
	 * @param Item The element to insert.
	 * @param Index Tells where to insert the new element.
	 * @return A reference to the newly-inserted element.
	 * @see Add, Remove
	 */
	[[nodiscard]] ElementType& Insert_GetRef(const ElementType& Item, SizeType Index) UE_LIFETIMEBOUND
	{
		CheckAddress(&Item);

		// construct a copy in place at Index (this new operator will insert at 
		// Index, then construct that memory with Item)
		InsertUninitializedImpl(Index);
		ElementType* Ptr = GetData() + Index;
		::new((void*)Ptr) ElementType(Item);
		return *Ptr;
	}

private:
	void RemoveAtImpl(SizeType Index)
	{
		ElementType* Dest = GetData() + Index;

		DestructItem(Dest);

		// Skip relocation in the common case that there is nothing to move.
		SizeType NumToMove = (ArrayNum - Index) - 1;
		if (NumToMove)
		{
			RelocateConstructItems<ElementType>((void*)Dest, Dest + 1, NumToMove);
		}
		--ArrayNum;

		SlackTrackerNumChanged();
	}

	void RemoveAtImpl(SizeType Index, SizeType Count)
	{
		ElementType* Dest = GetData() + Index;

		DestructItems(Dest, Count);

		// Skip relocation in the common case that there is nothing to move.
		SizeType NumToMove = (ArrayNum - Index) - Count;
		if (NumToMove)
		{
			RelocateConstructItems<ElementType>((void*)Dest, Dest + Count, NumToMove);
		}
		ArrayNum -= Count;

		SlackTrackerNumChanged();
	}

public:
	/**
	 * Removes an element at the given location, optionally shrinking the array.
	 *
	 * @param Index Location in array of the element to remove.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 */
	void RemoveAt(SizeType Index, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		RangeCheck(Index);
		RemoveAtImpl(Index);
		if (AllowShrinking == EAllowShrinking::Yes)
		{
			UE::Core::Private::ReallocShrink<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(
				sizeof(ElementType),
				alignof(ElementType),
				AllocatorInstance,
				ArrayNum,
				ArrayMax
			);
		}
	}

	/**
	 * Removes an element (or elements) at given location, optionally shrinking
	 * the array.
	 *
	 * @param Index Location in array of the element to remove.
	 * @param Count Number of elements to remove.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 */
	UE_FORCEINLINE_HINT void RemoveAt(SizeType Index, SizeType Count, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		RangeCheck(Index, Count);
		if (Count)
		{
			RemoveAtImpl(Index, (SizeType)Count);
			if (AllowShrinking == EAllowShrinking::Yes)
			{
				UE::Core::Private::ReallocShrink<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(
					sizeof(ElementType),
					alignof(ElementType),
					AllocatorInstance,
					ArrayNum,
					ArrayMax
				);
			}
		}
	}

private:
	void RemoveAtSwapImpl(SizeType Index)
	{
		ElementType* Data = GetData();
		ElementType* Dest = Data + Index;

		DestructItem(Dest);

		// Replace the elements in the hole created by the removal with elements from the end of the array, so the range of indices used by the array is contiguous.
		const SizeType NumElementsAfterHole = (ArrayNum - Index) - 1;
		const SizeType NumElementsToMoveIntoHole = FPlatformMath::Min(1, NumElementsAfterHole);
		if (NumElementsToMoveIntoHole)
		{
			RelocateConstructItems<ElementType>((void*)Dest, Data + (ArrayNum - NumElementsToMoveIntoHole), NumElementsToMoveIntoHole);
		}
		--ArrayNum;

		SlackTrackerNumChanged();
	}

	void RemoveAtSwapImpl(SizeType Index, SizeType Count)
	{
		ElementType* Data = GetData();
		ElementType* Dest = Data + Index;

		DestructItems(Dest, Count);

		// Replace the elements in the hole created by the removal with elements from the end of the array, so the range of indices used by the array is contiguous.
		const SizeType NumElementsAfterHole = (ArrayNum - Index) - Count;
		const SizeType NumElementsToMoveIntoHole = FPlatformMath::Min(Count, NumElementsAfterHole);
		if (NumElementsToMoveIntoHole)
		{
			RelocateConstructItems<ElementType>((void*)Dest, Data + (ArrayNum - NumElementsToMoveIntoHole), NumElementsToMoveIntoHole);
		}
		ArrayNum -= Count;

		SlackTrackerNumChanged();
	}

public:
	/**
	 * Removes an element at the given location, optionally shrinking the array.
	 *
	 * This version is much more efficient than RemoveAt (O(Count) instead of
	 * O(ArrayNum)), but does not preserve the order.
	 *
	 * @param Index Location in array of the element to remove.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 */
	UE_FORCEINLINE_HINT void RemoveAtSwap(SizeType Index, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		RangeCheck(Index);
		RemoveAtSwapImpl(Index);
		if (AllowShrinking == EAllowShrinking::Yes)
		{
			UE::Core::Private::ReallocShrink<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(
				sizeof(ElementType),
				alignof(ElementType),
				AllocatorInstance,
				ArrayNum,
				ArrayMax
			);
		}
	}

	/**
	 * Removes an element (or elements) at given location, optionally shrinking
	 * the array.
	 *
	 * This version is much more efficient than RemoveAt (O(Count) instead of
	 * O(ArrayNum)), but does not preserve the order.
	 *
	 * @param Index Location in array of the element to remove.
	 * @param Count Number of elements to remove.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 */
	UE_FORCEINLINE_HINT void RemoveAtSwap(SizeType Index, SizeType Count, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		RangeCheck(Index, Count);
		if (Count)
		{
			RemoveAtSwapImpl(Index, Count);
			if (AllowShrinking == EAllowShrinking::Yes)
			{
				UE::Core::Private::ReallocShrink<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(
					sizeof(ElementType),
					alignof(ElementType),
					AllocatorInstance,
					ArrayNum,
					ArrayMax
				);
			}
		}
	}

	/**
	 * Same as empty, but doesn't change memory allocations, unless the new size is larger than
	 * the current array. It calls the destructors on held items if needed and then zeros the ArrayNum.
	 *
	 * @param NewSize The expected usage size after calling this function.
	 */
	void Reset(SizeType NewSize = 0)
	{
		if (NewSize < 0)
		{
			// Cast to USizeType first to prevent sign extension on negative sizes, producing unusually large values.
			UE::Core::Private::OnInvalidArrayNum((unsigned long long)(USizeType)NewSize);
		}

		// If we have space to hold the expected size, then don't reallocate
		if (NewSize <= ArrayMax)
		{
			DestructItems(GetData(), ArrayNum);
			ArrayNum = 0;

			SlackTrackerNumChanged();
		}
		else
		{
			Empty(NewSize);
		}
	}

	/**
	 * Empties the array. It calls the destructors on held items if needed.
	 *
	 * @param Slack (Optional) The expected usage size after empty operation. Default is 0.
	 */
	void Empty(SizeType Slack = 0)
	{
		if (Slack < 0)
		{
			// Cast to USizeType first to prevent sign extension on negative sizes, producing unusually large values.
			UE::Core::Private::OnInvalidArrayNum((unsigned long long)(USizeType)Slack);
		}

		DestructItems(GetData(), ArrayNum);

		checkSlow(Slack >= 0);
		ArrayNum = 0;

		SlackTrackerNumChanged();

		if (ArrayMax != Slack)
		{
			UE::Core::Private::ReallocTo<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(
				sizeof(ElementType),
				alignof(ElementType),
				Slack,
				AllocatorInstance,
				ArrayNum,
				ArrayMax
			);
		}
	}

	/**
	 * Resizes array to given number of elements.
	 *
	 * @param NewNum New size of the array.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 */
	void SetNum(SizeType NewNum, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		if (NewNum > Num())
		{
			const SizeType Diff = NewNum - ArrayNum;
			const SizeType Index = AddUninitialized(Diff);
			DefaultConstructItems<ElementType>((void*)((uint8*)AllocatorInstance.GetAllocation() + Index * sizeof(ElementType)), Diff);
		}
		else if (NewNum < 0)
		{
			// Cast to USizeType first to prevent sign extension on negative sizes, producing unusually large values.
			UE::Core::Private::OnInvalidArrayNum((unsigned long long)(USizeType)NewNum);
		}
		else if (NewNum < Num())
		{
			RemoveAt(NewNum, Num() - NewNum, AllowShrinking);
		}
	}

	/**
	 * Resizes array to given number of elements, optionally shrinking it.
	 * New elements will be zeroed.
	 *
	 * @param NewNum New size of the array.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 */
	void SetNumZeroed(SizeType NewNum, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		if (NewNum > Num())
		{
			AddZeroed(NewNum - Num());
		}
		else if (NewNum < 0)
		{
			// Cast to USizeType first to prevent sign extension on negative sizes, producing unusually large values.
			UE::Core::Private::OnInvalidArrayNum((unsigned long long)(USizeType)NewNum);
		}
		else if (NewNum < Num())
		{
			RemoveAt(NewNum, Num() - NewNum, AllowShrinking);
		}
	}

	/**
	 * Resizes array to given number of elements. New elements will be uninitialized.
	 *
	 * @param NewNum New size of the array.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 */
	void SetNumUninitialized(SizeType NewNum, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		if (NewNum > Num())
		{
			AddUninitialized(NewNum - Num());
		}
		else if (NewNum < 0)
		{
			// Cast to USizeType first to prevent sign extension on negative sizes, producing unusually large values.
			UE::Core::Private::OnInvalidArrayNum((unsigned long long)(USizeType)NewNum);
		}
		else if (NewNum < Num())
		{
			RemoveAt(NewNum, Num() - NewNum, AllowShrinking);
		}
	}

	/**
	 * Does nothing except setting the new number of elements in the array. Does not destruct items, does not de-allocate memory.
	 * @param NewNum New number of elements in the array, must be <= the current number of elements in the array.
	 */
	void SetNumUnsafeInternal(SizeType NewNum)
	{
		checkSlow(NewNum <= Num() && NewNum >= 0);
		ArrayNum = NewNum;

		SlackTrackerNumChanged();
	}

	/**
	 * Pad the array with default values to ensure that it is at least the specified length. Does nothing if Num() >= DesiredNum.
	 *
	 * @param DesiredNum The desired number of elements that should exist in the array.
	 * @return The number of elements that were added to the array, or 0 if Num() >= DesiredNum.
	 */
	SizeType PadToNumDefaulted(SizeType DesiredNum)
	{
		SizeType Diff = DesiredNum - Num();
		if (Diff <= 0)
		{
			return 0;
		}

		AddDefaulted(Diff);

		return Diff;
	}

	/**
	 * Pad the array with the specified value to ensure that it is at least the specified length. Does nothing if Num() >= DesiredNum.
	 *
	 * @param DesiredNum The desired number of elements that should exist in the array.
	 * @param PadValue  The value to pad with
	 * @return The number of elements that were added to the array, or 0 if Num() >= DesiredNum.
	 */
	SizeType PadToNum(SizeType DesiredNum, const ElementType& PadValue)
	{
		SizeType Diff = DesiredNum - Num();
		if (Diff <= 0)
		{
			return 0;
		}

		SizeType Pos = AddUninitialized(Diff);
		ElementType* Data = GetData() + Pos;
		for (SizeType Index = 0; Index != Diff; ++Index, ++Data)
		{
			(void)new ((void*)Data) ElementType(PadValue);
		}

		return Diff;
	}

	/**
	 * Appends the specified array to this array.
	 *
	 * AllocatorType changing version.
	 *
	 * @param Source The array to append.
	 * @see Add, Insert
	 */
	template <typename OtherElementType, typename OtherAllocatorType>
	void Append(const TArray<OtherElementType, OtherAllocatorType>& Source)
	{
		check((void*)this != (void*)&Source);

		SizeType SourceCount = Source.Num();

		// Do nothing if the source is empty.
		if (!SourceCount)
		{
			return;
		}

		// Allocate memory for the new elements.
		SizeType Pos = AddUninitialized(SourceCount);
		ConstructItems<ElementType>((void*)(GetData() + Pos), Source.GetData(), SourceCount);
	}

	/**
	 * Appends the specified array to this array.
	 *
	 * @param Source The array to append.
	 * @see Add, Insert
	 */
	template <typename OtherElementType, typename OtherAllocator>
	void Append(TArray<OtherElementType, OtherAllocator>&& Source)
	{
		check((void*)this != (void*)&Source);

		SizeType SourceCount = Source.Num();

		// Do nothing if the source is empty.
		if (!SourceCount)
		{
			return;
		}

		// Allocate memory for the new elements.
		SizeType Pos = AddUninitialized(SourceCount);
		RelocateConstructItems<ElementType>((void*)(GetData() + Pos), Source.GetData(), SourceCount);
		Source.ArrayNum = 0;

		Source.SlackTrackerNumChanged();
	}

	/**
	 * Appends the elements from a contiguous range to this array.
	 *
	 * @param Source The range of elements to append.
	 * @see Add, Insert
	 */
	template <
		typename RangeType
		UE_REQUIRES(
			::TIsContiguousContainer<RangeType>::Value &&
			!UE::Core::Private::TIsTArrayOrDerivedFromTArray_V<std::remove_reference_t<RangeType>> &&
			UE::Core::Private::TArrayElementsAreCompatible_V<ElementType, TElementType_T<RangeType>>
		)
	>
	void Append(RangeType&& Source)
	{
		auto InCount = GetNum(Source);
		checkf((InCount >= 0) && ((sizeof(InCount) < sizeof(SizeType)) || (InCount <= static_cast<decltype(InCount)>(TNumericLimits<SizeType>::Max()))), TEXT("Invalid range size: %lld"), (long long)InCount);

		// Do nothing if the source is empty.
		if (!InCount)
		{
			return;
		}

		SizeType SourceCount = (SizeType)InCount;

		// Allocate memory for the new elements.
		SizeType Pos = AddUninitialized(SourceCount);
		ConstructItems<ElementType>((void*)(GetData() + Pos), GetDataHelper(Source), SourceCount);
	}

	/**
	 * Adds a raw array of elements to the end of the TArray.
	 *
	 * @param Ptr   A pointer to an array of elements to add.
	 * @param Count The number of elements to insert from Ptr.
	 * @see Add, Insert
	 */
	void Append(const ElementType* Ptr, SizeType Count)
	{
		check(Ptr != nullptr || Count == 0);

		SizeType Pos = AddUninitialized(Count);
		ConstructItems<ElementType>((void*)(GetData() + Pos), Ptr, Count);
	}

	/**
	 * Adds an initializer list of elements to the end of the TArray.
	 *
	 * @param InitList The initializer list of elements to add.
	 * @see Add, Insert
	 */
	UE_FORCEINLINE_HINT void Append(std::initializer_list<ElementType> InitList)
	{
		SizeType Count = (SizeType)InitList.size();

		SizeType Pos = AddUninitialized(Count);
		ConstructItems<ElementType>((void*)(GetData() + Pos), InitList.begin(), Count);
	}

	/**
	 * Appends the specified array to this array.
	 * Cannot append to self.
	 *
	 * Move semantics version.
	 *
	 * @param Other The array to append.
	 */
	UE_NODEBUG TArray& operator+=(TArray&& Other)
	{
		Append(MoveTemp(Other));
		return *this;
	}

	/**
	 * Appends the specified array to this array.
	 * Cannot append to self.
	 *
	 * @param Other The array to append.
	 */
	UE_NODEBUG TArray& operator+=(const TArray& Other)
	{
		Append(Other);
		return *this;
	}

	/**
	 * Appends the specified initializer list to this array.
	 *
	 * @param InitList The initializer list to append.
	 */
	UE_NODEBUG TArray& operator+=(std::initializer_list<ElementType> InitList)
	{
		Append(InitList);
		return *this;
	}

	/**
	 * Constructs a new item at the end of the array, possibly reallocating the whole array to fit.
	 *
	 * @param Args	The arguments to forward to the constructor of the new item.
	 * @return		Index to the new item
	 */
	template <typename... ArgsType>
	UE_FORCEINLINE_HINT SizeType Emplace(ArgsType&&... Args)
	{
		// If this fails to compile when trying to call Emplace with a non-public constructor,
		// do not make TArray a friend.
		//
		// Instead, prefer this pattern:
		//
		//     class FMyType
		//     {
		//     private:
		//         struct FPrivateToken { explicit FPrivateToken() = default; };
		//
		//     public:
		//         // This has an equivalent access level to a private constructor,
		//         // as only friends of FMyType will have access to FPrivateToken,
		//         // but Emplace can legally call it since it's public.
		//         explicit FMyType(FPrivateToken, int32 Int, float Real, const TCHAR* String);
		//     };
		//
		//     TArray<FMyType> Arr:
		//
		//     // Won't compile if the caller doesn't have access to FMyType::FPrivateToken
		//     Arr.Emplace(FMyType::FPrivateToken{}, 5, 3.14f, TEXT("Banana"));
		//

		// Begin sensitive code! See comments in AddUninitialized() before touching!
		if (ArrayNum == ArrayMax)
		{
			if constexpr (sizeof(ElementType) <= 255 && alignof(ElementType) <= 255) // -V590 
			{
				ArrayNum = UE::Core::Private::ReallocGrow1_DoAlloc_Tiny<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(sizeof(ElementType) | (alignof(ElementType) << 8), AllocatorInstance, ArrayMax);
			}
			else
			{
				ArrayNum = UE::Core::Private::ReallocGrow1_DoAlloc<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(sizeof(ElementType), alignof(ElementType), AllocatorInstance, ArrayMax);
			}
		}
		// End sensitive code!
		SizeType OldArrayNum = ArrayNum;
		ArrayNum++;		
		void* Ptr = (char*)AllocatorInstance.GetAllocation() + sizeof(ElementType) * OldArrayNum;
		(void)new (Ptr) ElementType(Forward<ArgsType>(Args)...);
		return OldArrayNum;
	}

	/**
	 * Constructs a new item at the end of the array, possibly reallocating the whole array to fit.
	 *
	 * @param Args	The arguments to forward to the constructor of the new item.
	 * @return A reference to the newly-inserted element.
	 */
	template <typename... ArgsType>
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& Emplace_GetRef(ArgsType&&... Args) UE_LIFETIMEBOUND
	{
		// If this fails to compile when trying to call Emplace with a non-public constructor,
		// do not make TArray a friend.
		//
		// Instead, prefer this pattern:
		//
		//     class FMyType
		//     {
		//     private:
		//         struct FPrivateToken { explicit FPrivateToken() = default; };
		//
		//     public:
		//         // This has an equivalent access level to a private constructor,
		//         // as only friends of FMyType will have access to FPrivateToken,
		//         // but Emplace can legally call it since it's public.
		//         explicit FMyType(FPrivateToken, int32 Int, float Real, const TCHAR* String);
		//     };
		//
		//     TArray<FMyType> Arr:
		//
		//     // Won't compile if the caller doesn't have access to FMyType::FPrivateToken
		//     Arr.Emplace(FMyType::FPrivateToken{}, 5, 3.14f, TEXT("Banana"));
		//
		// Begin sensitive code! See comments in AddUninitialized() before touching!
		if (ArrayNum == ArrayMax)
		{
			if constexpr (sizeof(ElementType) <= 255 && alignof(ElementType) <= 255) // -V590 
			{
				// Looks weird but saves the register reload
				ArrayNum = UE::Core::Private::ReallocGrow1_DoAlloc_Tiny<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(sizeof(ElementType) | (alignof(ElementType) << 8), AllocatorInstance, ArrayMax);
			}
			else
			{
				// Looks weird but saves the register reload
				ArrayNum = UE::Core::Private::ReallocGrow1_DoAlloc<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(sizeof(ElementType), alignof(ElementType), AllocatorInstance, ArrayMax);
			}
		}
		// End sensitive code!
		SizeType OldArrayNum = ArrayNum;
		ArrayNum++;
		void* Ptr = (char*)AllocatorInstance.GetAllocation() + sizeof(ElementType) * OldArrayNum;
		return *new (Ptr) ElementType(Forward<ArgsType>(Args)...);
	}

	/**
	 * Constructs a new item at a specified index, possibly reallocating the whole array to fit.
	 *
	 * @param Index	The index to add the item at.
	 * @param Args	The arguments to forward to the constructor of the new item.
	 */
	template <typename... ArgsType>
	UE_FORCEINLINE_HINT void EmplaceAt(SizeType Index, ArgsType&&... Args)
	{
		InsertUninitializedImpl(Index, 1);
		::new((void*)(GetData() + Index)) ElementType(Forward<ArgsType>(Args)...);
	}

	/**
	 * Constructs a new item at a specified index, possibly reallocating the whole array to fit.
	 *
	 * @param Index	The index to add the item at.
	 * @param Args	The arguments to forward to the constructor of the new item.
	 * @return A reference to the newly-inserted element.
	 */
	template <typename... ArgsType>
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& EmplaceAt_GetRef(SizeType Index, ArgsType&&... Args) UE_LIFETIMEBOUND
	{
		InsertUninitializedImpl(Index, 1);
		ElementType* Ptr = GetData() + Index;
		::new((void*)Ptr) ElementType(Forward<ArgsType>(Args)...);
		return *Ptr;
	}

	/**
	 * Adds a new item to the end of the array, possibly reallocating the whole array to fit.
	 *
	 * Move semantics version.
	 *
	 * @param Item The item to add
	 * @return Index to the new item
	 * @see AddDefaulted, AddUnique, AddZeroed, Append, Insert
	 */
	UE_NODEBUG UE_FORCEINLINE_HINT SizeType Add(ElementType&& Item)
	{
		CheckAddress(&Item);
		return Emplace(MoveTempIfPossible(Item));
	}

	/**
	 * Adds a new item to the end of the array, possibly reallocating the whole array to fit.
	 *
	 * @param Item The item to add
	 * @return Index to the new item
	 * @see AddDefaulted, AddUnique, AddZeroed, Append, Insert
	 */
	UE_NODEBUG UE_FORCEINLINE_HINT SizeType Add(const ElementType& Item)
	{
		CheckAddress(&Item);
		return Emplace(Item);
	}

	/**
	 * Adds a new item to the end of the array, possibly reallocating the whole array to fit.
	 *
	 * Move semantics version.
	 *
	 * @param Item The item to add
	 * @return A reference to the newly-inserted element.
	 * @see AddDefaulted_GetRef, AddUnique_GetRef, AddZeroed_GetRef, Insert_GetRef
	 */
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT ElementType& Add_GetRef(ElementType&& Item) UE_LIFETIMEBOUND
	{
		CheckAddress(&Item);
		return Emplace_GetRef(MoveTempIfPossible(Item));
	}

	/**
	 * Adds a new item to the end of the array, possibly reallocating the whole array to fit.
	 *
	 * @param Item The item to add
	 * @return A reference to the newly-inserted element.
	 * @see AddDefaulted_GetRef, AddUnique_GetRef, AddZeroed_GetRef, Insert_GetRef
	 */
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT ElementType& Add_GetRef(const ElementType& Item) UE_LIFETIMEBOUND
	{
		CheckAddress(&Item);
		return Emplace_GetRef(Item);
	}

	/**
	 * Adds new items to the end of the array, possibly reallocating the whole
	 * array to fit. The new items will be zeroed.
	 *
	 * Caution, AddZeroed() will create elements without calling the
	 * constructor and this is not appropriate for element types that require
	 * a constructor to function properly.
	 *
	 * @param  Count  The number of new items to add.
	 * @return Index to the first of the new items.
	 * @see Add, AddDefaulted, AddUnique, Append, Insert
	 */
	SizeType AddZeroed()
	{
		const SizeType Index = AddUninitialized();
		FMemory::Memzero((uint8*)AllocatorInstance.GetAllocation() + Index * sizeof(ElementType), sizeof(ElementType));
		return Index;
	}
	SizeType AddZeroed(SizeType Count)
	{
		const SizeType Index = AddUninitialized(Count);
		FMemory::Memzero((uint8*)AllocatorInstance.GetAllocation() + Index*sizeof(ElementType), Count*sizeof(ElementType));
		return Index;
	}

	/**
	 * Adds a new item to the end of the array, possibly reallocating the whole
	 * array to fit. The new item will be zeroed.
	 *
	 * Caution, AddZeroed_GetRef() will create elements without calling the
	 * constructor and this is not appropriate for element types that require
	 * a constructor to function properly.
	 *
	 * @return A reference to the newly-inserted element.
	 * @see Add_GetRef, AddDefaulted_GetRef, AddUnique_GetRef, Insert_GetRef
	 */
	[[nodiscard]] ElementType& AddZeroed_GetRef() UE_LIFETIMEBOUND
	{
		const SizeType Index = AddUninitialized();
		ElementType* Ptr = GetData() + Index;
		FMemory::Memzero(Ptr, sizeof(ElementType));
		return *Ptr;
	}

	/**
	 * Adds new items to the end of the array, possibly reallocating the whole
	 * array to fit. The new items will be default-constructed.
	 *
	 * @param  Count  The number of new items to add.
	 * @return Index to the first of the new items.
	 * @see Add, AddZeroed, AddUnique, Append, Insert
	 */
	SizeType AddDefaulted()
	{
		const SizeType Index = AddUninitialized();
		DefaultConstructItems<ElementType>((void*)((uint8*)AllocatorInstance.GetAllocation() + Index * sizeof(ElementType)), 1);
		return Index;
	}
	SizeType AddDefaulted(SizeType Count)
	{
		const SizeType Index = AddUninitialized(Count);
		DefaultConstructItems<ElementType>((void*)((uint8*)AllocatorInstance.GetAllocation() + Index * sizeof(ElementType)), Count);
		return Index;
	}

	/**
	 * Add a new item to the end of the array, possibly reallocating the whole
	 * array to fit. The new item will be default-constructed.
	 *
	 * @return A reference to the newly-inserted element.
	 * @see Add_GetRef, AddZeroed_GetRef, AddUnique_GetRef, Insert_GetRef
	 */
	[[nodiscard]] ElementType& AddDefaulted_GetRef() UE_LIFETIMEBOUND
	{
		const SizeType Index = AddUninitialized();
		ElementType* Ptr = GetData() + Index;
		DefaultConstructItems<ElementType>((void*)Ptr, 1);
		return *Ptr;
	}

#if !UE_DEPRECATE_MUTABLE_TOBJECTPTR
	/** Mutable implicit conversion operator to container of compatible element type. */
	template <
		typename AliasElementType = ElementType
		UE_REQUIRES(TIsContainerElementTypeReinterpretable_V<AliasElementType>)
	>
	[[nodiscard]] operator TArray<typename TContainerElementTypeCompatibility<AliasElementType>::ReinterpretType, AllocatorType>& ()
	{
		using ElementCompat = TContainerElementTypeCompatibility<ElementType>;
		ElementCompat::ReinterpretRangeContiguous(begin(), end(), Num());
		return *reinterpret_cast<TArray<typename ElementCompat::ReinterpretType>*>(this);
	}
#endif
  
	/** Immutable implicit conversion operator to constant container of compatible element type. */
	template <
		typename AliasElementType = ElementType
		UE_REQUIRES(TIsContainerElementTypeReinterpretable_V<AliasElementType>)
	>
	[[nodiscard]] operator const TArray<typename TContainerElementTypeCompatibility<AliasElementType>::ReinterpretType, AllocatorType>& () const
	{
		using ElementCompat = TContainerElementTypeCompatibility<ElementType>;
		ElementCompat::ReinterpretRangeContiguous(begin(), end(), Num());
		return *reinterpret_cast<const TArray<typename ElementCompat::ReinterpretType>*>(this);
	}

	/**
	 * Move assignment operator.
	 * Compatible element type version.
	 *
	 * @param Other Array to assign and move from.
	 */
	template <
		typename AliasElementType = ElementType
		UE_REQUIRES(TIsContainerElementTypeCopyable_V<AliasElementType>)
	>
	TArray& operator=(TArray<typename TContainerElementTypeCompatibility<ElementType>::CopyFromOtherType, AllocatorType>&& Other)
	{
		TContainerElementTypeCompatibility<ElementType>::CopyingFromOtherType();
		DestructItems(GetData(), ArrayNum);
		MoveOrCopy(*this, Other, ArrayMax);
		return *this;
	}

	/**
	 * Assignment operator. First deletes all currently contained elements
	 * and then copies from other array.
	 * Compatible element type version.
	 *
	 * @param Other The source array to assign from.
	 */
	template <
		typename OtherAllocator,
		typename AliasElementType = ElementType
		UE_REQUIRES(TIsContainerElementTypeCopyable_V<AliasElementType>)
	>
	TArray& operator=(const TArray<typename TContainerElementTypeCompatibility<ElementType>::CopyFromOtherType, OtherAllocator>& Other)
	{
		TContainerElementTypeCompatibility<ElementType>::CopyingFromOtherType();
		DestructItems(GetData(), ArrayNum);
		CopyToEmpty(Other.GetData(), Other.Num(), ArrayMax);
		return *this;
	}

	/**
	 * Inserts given elements into the array at given location.
	 * Compatible element type version.
	 *
	 * @param Items Array of elements to insert.
	 * @param InIndex Tells where to insert the new elements.
	 * @returns Location at which the item was inserted.
	 */
	template <
		typename OtherAllocator,
		typename AliasElementType = ElementType
		UE_REQUIRES(TIsContainerElementTypeCopyable_V<AliasElementType>)
	>
	SizeType Insert(const TArray<typename TContainerElementTypeCompatibility<ElementType>::CopyFromOtherType, OtherAllocator>& Items, const SizeType InIndex)
	{
		TContainerElementTypeCompatibility<ElementType>::CopyingFromOtherType();

		auto NumNewElements = Items.Num();

		InsertUninitializedImpl(InIndex, NumNewElements);
		ConstructItems<ElementType>((void*)(GetData() + InIndex), Items.GetData(), NumNewElements);

		return InIndex;
	}

	/**
	 * Inserts given elements into the array at given location.
	 * Compatible element type version.
	 *
	 * @param Items Array of elements to insert.
	 * @param InIndex Tells where to insert the new elements.
	 * @returns Location at which the item was inserted.
	 */
	template <
		typename OtherAllocator,
		typename AliasElementType = ElementType
		UE_REQUIRES(TIsContainerElementTypeCopyable_V<AliasElementType>)
	>
	SizeType Insert(TArray<typename TContainerElementTypeCompatibility<ElementType>::CopyFromOtherType, OtherAllocator>&& Items, const SizeType InIndex)
	{
		check((const void*)this != (const void*)&Items);
		TContainerElementTypeCompatibility<ElementType>::CopyingFromOtherType();

		auto NumNewElements = Items.Num();

		InsertUninitializedImpl(InIndex, NumNewElements);
		RelocateConstructItems<ElementType>((void*)(GetData() + InIndex), Items.GetData(), NumNewElements);
		Items.ArrayNum = 0;

		Items.SlackTrackerNumChanged();

		return InIndex;
	}

	/**
	 * Adds a raw array of elements to the end of the TArray.
	 * Compatible element type version.
	 *
	 * @param Ptr   A pointer to an array of elements to add.
	 * @param Count The number of elements to insert from Ptr.
	 * @see Add, Insert
	 */
	template <
		typename AliasElementType = ElementType
		UE_REQUIRES(TIsContainerElementTypeCopyable_V<AliasElementType>)
	>
	void Append(const typename TContainerElementTypeCompatibility<ElementType>::CopyFromOtherType* Ptr, SizeType Count)
	{
		check(Ptr != nullptr || Count == 0);
		TContainerElementTypeCompatibility<ElementType>::CopyingFromOtherType();

		SizeType Pos = AddUninitialized(Count);
		ConstructItems<ElementType>((void*)(GetData() + Pos), Ptr, Count);
	}

private:

	/**
	 * Adds unique element to array if it doesn't exist.
	 *
	 * @param Args Item to add.
	 * @returns Index of the element in the array.
	 */
	template <typename ArgsType>
	SizeType AddUniqueImpl(ArgsType&& Args)
	{
		SizeType Index;
		if (Find(Args, Index))
		{
			return Index;
		}

		return Add(Forward<ArgsType>(Args));
	}

public:

	/**
	 * Adds unique element to array if it doesn't exist.
	 *
	 * Move semantics version.
	 *
	 * @param Item Item to add.
	 * @returns Index of the element in the array.
	 * @see Add, AddDefaulted, AddZeroed, Append, Insert
	 */
	UE_FORCEINLINE_HINT SizeType AddUnique(ElementType&& Item)
	{
		return AddUniqueImpl(MoveTempIfPossible(Item));
	}

	/**
	 * Adds unique element to array if it doesn't exist.
	 *
	 * @param Item Item to add.
	 * @returns Index of the element in the array.
	 * @see Add, AddDefaulted, AddZeroed, Append, Insert
	 */
	UE_FORCEINLINE_HINT SizeType AddUnique(const ElementType& Item)
	{
		return AddUniqueImpl(Item);
	}

	/**
	 * Reserves memory such that the array can contain at least Number elements.
	 *
	 * @param Number The number of elements that the array should be able to contain after allocation.
	 * @see Shrink
	 */
	UE_FORCEINLINE_HINT void Reserve(SizeType Number)
	{
		checkSlow(Number >= 0);
		if (Number < 0)
		{
			// Cast to USizeType first to prevent sign extension on negative sizes, producing unusually large values.
			UE::Core::Private::OnInvalidArrayNum((unsigned long long)(USizeType)Number);
		}
		else if (Number > ArrayMax)
		{
			UE::Core::Private::ReallocTo<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(
				sizeof(ElementType),
				alignof(ElementType),
				Number,
				AllocatorInstance,
				ArrayNum,
				ArrayMax
			);
		}
	}

	/**
	 * Sets the size of the array, filling it with the given element.
	 *
	 * @param Element The element to fill array with.
	 * @param Number The number of elements that the array should be able to contain after allocation.
	 */
	void Init(const ElementType& Element, SizeType Number)
	{
		Empty(Number);
		for (SizeType Index = 0; Index < Number; ++Index)
		{
			Add(Element);
		}
	}

	/**
	 * Removes the first occurrence of the specified item in the array,
	 * maintaining order but not indices.
	 *
	 * @param Item The item to remove.
	 * @returns The number of items removed. For RemoveSingleItem, this is always either 0 or 1.
	 * @see Add, Insert, Remove, RemoveAll, RemoveAllSwap
	 */
	SizeType RemoveSingle(const ElementType& Item)
	{
		SizeType Index = Find(Item);
		if (Index == INDEX_NONE)
		{
			return 0;
		}

		auto* RemovePtr = GetData() + Index;

		// Destruct items that match the specified Item.
		DestructItems(RemovePtr, 1);
		RelocateConstructItems<ElementType>((void*)RemovePtr, RemovePtr + 1, ArrayNum - (Index + 1));

		// Update the array count
		--ArrayNum;

		SlackTrackerNumChanged();

		// Removed one item
		return 1;
	}

	/**
	 * Removes as many instances of Item as there are in the array, maintaining
	 * order but not indices.
	 *
	 * @param Item Item to remove from array.
	 * @returns Number of removed elements.
	 * @see Add, Insert, RemoveAll, RemoveAllSwap, RemoveSingle, RemoveSwap
	 */
	SizeType Remove(const ElementType& Item)
	{
		CheckAddress(&Item);

		// Element is non-const to preserve compatibility with existing code with a non-const operator==() member function
		return RemoveAll([&Item](ElementType& Element) { return Element == Item; });
	}

	/**
	 * Remove all instances that match the predicate, maintaining order but not indices
	 * Optimized to work with runs of matches/non-matches
	 *
	 * @param Predicate Predicate class instance
	 * @returns Number of removed elements.
	 * @see Add, Insert, RemoveAllSwap, RemoveSingle, RemoveSwap
	 */
	template <class PREDICATE_CLASS>
	SizeType RemoveAll(const PREDICATE_CLASS& Predicate)
	{
		const SizeType OriginalNum = ArrayNum;
		if (!OriginalNum)
		{
			return 0; // nothing to do, loop assumes one item so need to deal with this edge case here
		}

		ElementType* Data = GetData();

		SizeType WriteIndex = 0;
		SizeType ReadIndex = 0;
		bool bNotMatch = !::Invoke(Predicate, Data[ReadIndex]); // use a ! to guarantee it can't be anything other than zero or one
		do
		{
			SizeType RunStartIndex = ReadIndex++;
			while (ReadIndex < OriginalNum && bNotMatch == !::Invoke(Predicate, Data[ReadIndex]))
			{
				ReadIndex++;
			}
			SizeType RunLength = ReadIndex - RunStartIndex;
			checkSlow(RunLength > 0);
			if (bNotMatch)
			{
				// this was a non-matching run, we need to move it
				if (WriteIndex != RunStartIndex)
				{
					RelocateConstructItems<ElementType>((void*)(Data + WriteIndex), Data + RunStartIndex, RunLength);
				}
				WriteIndex += RunLength;
			}
			else
			{
				// this was a matching run, delete it
				DestructItems(Data + RunStartIndex, RunLength);
			}
			bNotMatch = !bNotMatch;
		} while (ReadIndex < OriginalNum);

		ArrayNum = WriteIndex;

		SlackTrackerNumChanged();

		return OriginalNum - ArrayNum;
	}

	/**
	 * Remove all instances that match the predicate
	 *
	 * @param Predicate Predicate class instance
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 * @see Remove, RemoveSingle, RemoveSingleSwap, RemoveSwap
	 */
	template <class PREDICATE_CLASS>
	SizeType RemoveAllSwap(const PREDICATE_CLASS& Predicate, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		bool bRemoved = false;
		const SizeType OriginalNum = ArrayNum;
		for (SizeType ItemIndex = 0; ItemIndex < Num();)
		{
			if (::Invoke(Predicate, (*this)[ItemIndex]))
			{
				bRemoved = true;
				RemoveAtSwap(ItemIndex, EAllowShrinking::No);
			}
			else
			{
				++ItemIndex;
			}
		}

		if (bRemoved && AllowShrinking == EAllowShrinking::Yes)
		{
			UE::Core::Private::ReallocShrink<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(
				sizeof(ElementType),
				alignof(ElementType),
				AllocatorInstance,
				ArrayNum,
				ArrayMax
			);
		}

		return OriginalNum - ArrayNum;
	}

	/**
	 * Removes the first occurrence of the specified item in the array. This version is much more efficient
	 * O(Count) instead of O(ArrayNum), but does not preserve the order
	 *
	 * @param Item The item to remove
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 *
	 * @returns The number of items removed. For RemoveSingleItem, this is always either 0 or 1.
	 * @see Add, Insert, Remove, RemoveAll, RemoveAllSwap, RemoveSwap
	 */
	SizeType RemoveSingleSwap(const ElementType& Item, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		SizeType Index = Find(Item);
		if (Index == INDEX_NONE)
		{
			return 0;
		}

		RemoveAtSwap(Index, 1, AllowShrinking);

		// Removed one item
		return 1;
	}

	/**
	 * Removes all instances of a given item from the array.
	 *
	 * This version is much more efficient, because it uses RemoveAtSwap
	 * internally which is O(Count) instead of RemoveAt which is O(ArrayNum),
	 * but does not preserve the order.
	 *
	 * @param Item The item to remove
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 *
	 * @returns Number of elements removed.
	 * @see Add, Insert, Remove, RemoveAll, RemoveAllSwap
	 */
	SizeType RemoveSwap(const ElementType& Item, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		CheckAddress(&Item);

		const SizeType OriginalNum = ArrayNum;
		bool bRemoved = false;
		for (SizeType Index = 0; Index < ArrayNum; Index++)
		{
			if ((*this)[Index] == Item)
			{
				bRemoved = true;
				RemoveAtSwap(Index--, EAllowShrinking::No);
			}
		}

		if (bRemoved && AllowShrinking == EAllowShrinking::Yes)
		{
			UE::Core::Private::ReallocShrink<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(
				sizeof(ElementType),
				alignof(ElementType),
				AllocatorInstance,
				ArrayNum,
				ArrayMax
			);
		}

		return OriginalNum - ArrayNum;
	}

	/**
	 * Element-wise array memory swap.
	 *
	 * @param FirstIndexToSwap Position of the first element to swap.
	 * @param SecondIndexToSwap Position of the second element to swap.
	 */
	UE_FORCEINLINE_HINT void SwapMemory(SizeType FirstIndexToSwap, SizeType SecondIndexToSwap)
	{
		::Swap(
			*(ElementType*)((uint8*)AllocatorInstance.GetAllocation() + (sizeof(ElementType)*FirstIndexToSwap)),
			*(ElementType*)((uint8*)AllocatorInstance.GetAllocation() + (sizeof(ElementType)*SecondIndexToSwap))
		);
	}

	/**
	 * Element-wise array element swap.
	 *
	 * This version is doing more sanity checks than SwapMemory.
	 *
	 * @param FirstIndexToSwap Position of the first element to swap.
	 * @param SecondIndexToSwap Position of the second element to swap.
	 */
	UE_FORCEINLINE_HINT void Swap(SizeType FirstIndexToSwap, SizeType SecondIndexToSwap)
	{
		check((FirstIndexToSwap >= 0) && (SecondIndexToSwap >= 0));
		check((ArrayNum > FirstIndexToSwap) && (ArrayNum > SecondIndexToSwap));
		if (FirstIndexToSwap != SecondIndexToSwap)
		{
			SwapMemory(FirstIndexToSwap, SecondIndexToSwap);
		}
	}

	/**
	 * Searches for the first entry of the specified type, will only work with
	 * TArray<UObject*>. Optionally return the item's index, and can specify
	 * the start index.
	 *
	 * @param Item (Optional output) If it's not null, then it will point to
	 *             the found element. Untouched if element hasn't been found.
	 * @param ItemIndex (Optional output) If it's not null, then it will be set
	 *             to the position of found element in the array. Untouched if
	 *             element hasn't been found.
	 * @param StartIndex (Optional) Index in array at which the function should
	 *             start to look for element.
	 * @returns True if element was found. False otherwise.
	 */
	template<typename SearchType>
	bool FindItemByClass(SearchType **Item = nullptr, SizeType *ItemIndex = nullptr, SizeType StartIndex = 0) const
	{
		UClass* SearchClass = SearchType::StaticClass();
		for (SizeType Idx = StartIndex; Idx < ArrayNum; Idx++)
		{
			if ((*this)[Idx] != nullptr && (*this)[Idx]->IsA(SearchClass))
			{
				if (Item != nullptr)
				{
					*Item = (SearchType*)((*this)[Idx]);
				}
				if (ItemIndex != nullptr)
				{
					*ItemIndex = Idx;
				}
				return true;
			}
		}
		return false;
	}

	// Iterators
	using TIterator      = TIndexedContainerIterator<      TArray,       ElementType, SizeType>;
	using TConstIterator = TIndexedContainerIterator<const TArray, const ElementType, SizeType>;

	/**
	 * Creates an iterator for the contents of this array
	 *
	 * @returns The iterator.
	 */
	UE_NODEBUG [[nodiscard]] TIterator CreateIterator()
	{
		return TIterator(*this);
	}

	/**
	 * Creates a const iterator for the contents of this array
	 *
	 * @returns The const iterator.
	 */
	UE_NODEBUG [[nodiscard]] TConstIterator CreateConstIterator() const
	{
		return TConstIterator(*this);
	}

	#if TARRAY_RANGED_FOR_CHECKS
		using RangedForIteratorType             = TCheckedPointerIterator<      ElementType, SizeType, false>;
		using RangedForConstIteratorType        = TCheckedPointerIterator<const ElementType, SizeType, false>;
		using RangedForReverseIteratorType      = TCheckedPointerIterator<      ElementType, SizeType, true>;
		using RangedForConstReverseIteratorType = TCheckedPointerIterator<const ElementType, SizeType, true>;
	#else
		using RangedForIteratorType             =                               ElementType*;
		using RangedForConstIteratorType        =                         const ElementType*;
		using RangedForReverseIteratorType      = TReversePointerIterator<      ElementType>;
		using RangedForConstReverseIteratorType = TReversePointerIterator<const ElementType>;
	#endif

public:

	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	#if TARRAY_RANGED_FOR_CHECKS
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForIteratorType             begin ()       { return RangedForIteratorType            (ArrayNum, GetData()); }
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForConstIteratorType        begin () const { return RangedForConstIteratorType       (ArrayNum, GetData()); }
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForIteratorType             end   ()       { return RangedForIteratorType            (ArrayNum, GetData() + Num()); }
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForConstIteratorType        end   () const { return RangedForConstIteratorType       (ArrayNum, GetData() + Num()); }
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForReverseIteratorType      rbegin()       { return RangedForReverseIteratorType     (ArrayNum, GetData() + Num()); }
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForConstReverseIteratorType rbegin() const { return RangedForConstReverseIteratorType(ArrayNum, GetData() + Num()); }
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForReverseIteratorType      rend  ()       { return RangedForReverseIteratorType     (ArrayNum, GetData()); }
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForConstReverseIteratorType rend  () const { return RangedForConstReverseIteratorType(ArrayNum, GetData()); }
	#else
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForIteratorType             begin ()       { return                                   GetData(); }
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForConstIteratorType        begin () const { return                                   GetData(); }
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForIteratorType             end   ()       { return                                   GetData() + Num(); }
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForConstIteratorType        end   () const { return                                   GetData() + Num(); }
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForReverseIteratorType      rbegin()       { return RangedForReverseIteratorType     (GetData() + Num()); }
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForConstReverseIteratorType rbegin() const { return RangedForConstReverseIteratorType(GetData() + Num()); }
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForReverseIteratorType      rend  ()       { return RangedForReverseIteratorType     (GetData()); }
		[[nodiscard]] UE_NODEBUG UE_FORCEINLINE_HINT RangedForConstReverseIteratorType rend  () const { return RangedForConstReverseIteratorType(GetData()); }
	#endif

public:

	/**
	 * Sorts the array assuming < operator is defined for the item type.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during sorting.
	 *        Therefore, your array will be sorted by the values being pointed to, rather than the pointers' values.
	 *        If this is not desirable, please use Algo::Sort(MyArray) directly instead.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	UE_NODEBUG void Sort()
	{
		Algo::Sort(*this, TDereferenceWrapper<ElementType, TLess<>>(TLess<>()));
	}

	/**
	 * Sorts the array using user define predicate class.
	 *
	 * @param Predicate Predicate class instance.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during sorting.
	 *        Therefore, your predicate will be passed references rather than pointers.
	 *        If this is not desirable, please use Algo::Sort(MyArray, Predicate) directly instead.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	template <class PREDICATE_CLASS>
	UE_NODEBUG void Sort(const PREDICATE_CLASS& Predicate)
	{
		TDereferenceWrapper<ElementType, PREDICATE_CLASS> PredicateWrapper(Predicate);
		Algo::Sort(*this, PredicateWrapper);
	}

	/**
	 * Stable sorts the array assuming < operator is defined for the item type.
	 *
	 * Stable sort is slower than non-stable algorithm.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during sorting.
	 *        Therefore, your array will be sorted by the values being pointed to, rather than the pointers' values.
	 *        If this is not desirable, please use Algo::StableSort(MyArray) directly instead.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	UE_NODEBUG void StableSort()
	{
		Algo::StableSort(*this, TDereferenceWrapper<ElementType, TLess<>>(TLess<>()));
	}

	/**
	 * Stable sorts the array using user defined predicate class.
	 *
	 * Stable sort is slower than non-stable algorithm.
	 *
	 * @param Predicate Predicate class instance
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during sorting.
	 *        Therefore, your predicate will be passed references rather than pointers.
	 *        If this is not desirable, please use Algo::StableSort(MyArray, Predicate) directly instead.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	template <class PREDICATE_CLASS>
	UE_NODEBUG void StableSort(const PREDICATE_CLASS& Predicate)
	{
		TDereferenceWrapper<ElementType, PREDICATE_CLASS> PredicateWrapper(Predicate);
		Algo::StableSort(*this, PredicateWrapper);
	}

#if defined(_MSC_VER) && !defined(__clang__)	// Relies on MSVC-specific lazy template instantiation to support arrays of incomplete types
private:
	/**
	 * Helper function that can be used inside the debuggers watch window to debug TArrays. E.g. "*Class->Defaults.DebugGet(5)". 
	 *
	 * @param Index Position to get.
	 * @returns Reference to the element at given position.
	 */
	[[nodiscard]] FORCENOINLINE const ElementType& DebugGet(SizeType Index) const
	{
		return GetData()[Index];
	}
#endif

private:
	/**
	 * Copies data from one array into this array. Uses the fast path if the
	 * data in question does not need a constructor.
	 *
	 * @param Source The source array to copy
	 * @param PrevMax The previous allocated size
	 */
	template <typename OtherElementType, typename OtherSizeType>
	void CopyToEmpty(const OtherElementType* OtherData, OtherSizeType OtherNum, SizeType PrevMax)
	{
		SizeType NewNum = (SizeType)OtherNum;
		checkf((OtherSizeType)NewNum == OtherNum, TEXT("Invalid number of elements to add to this array type: %lld"), (long long)NewNum);

		ArrayNum = NewNum;
		if (OtherNum || PrevMax)
		{
			UE::Core::Private::ReallocForCopy<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(
				sizeof(ElementType),
				alignof(ElementType),
				NewNum,
				PrevMax,
				AllocatorInstance,
				ArrayNum,
				ArrayMax
			);
			ConstructItems<ElementType>((void*)GetData(), OtherData, OtherNum);
		}
		else
		{
			ArrayMax = AllocatorInstance.GetInitialCapacity();
		}

		SlackTrackerNumChanged();
	}

	/**
	 * Copies data from one array into this array. Uses the fast path if the
	 * data in question does not need a constructor.
	 *
	 * @param Source The source array to copy
	 * @param PrevMax The previous allocated size
	 * @param ExtraSlack Additional amount of memory to allocate at
	 *                   the end of the buffer. Counted in elements.
	 */
	template <typename OtherElementType, typename OtherSizeType>
	void CopyToEmptyWithSlack(const OtherElementType* OtherData, OtherSizeType OtherNum, SizeType PrevMax, SizeType ExtraSlack)
	{
		SizeType NewNum = (SizeType)OtherNum;
		checkf((OtherSizeType)NewNum == OtherNum, TEXT("Invalid number of elements to add to this array type: %lld"), (long long)NewNum);

		ArrayNum = NewNum;
		if (OtherNum || ExtraSlack || PrevMax)
		{
			USizeType NewMax = NewNum + ExtraSlack;

			// This should only happen when we've underflowed or overflowed SizeType
			if ((SizeType)NewMax < NewNum)
			{
				UE::Core::Private::OnInvalidArrayNum((unsigned long long)NewMax);
			}

			UE::Core::Private::ReallocForCopy<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(
				sizeof(ElementType),
				alignof(ElementType),
				NewNum + ExtraSlack,
				PrevMax,
				AllocatorInstance,
				ArrayNum,
				ArrayMax
			);
			ConstructItems<ElementType>((void*)GetData(), OtherData, OtherNum);
		}
		else
		{
			ArrayMax = AllocatorInstance.GetInitialCapacity();
		}

		SlackTrackerNumChanged();
	}

protected:

	template<typename ElementType, typename AllocatorType>
	friend class TIndirectArray;

	ElementAllocatorType AllocatorInstance;
	SizeType             ArrayNum;
	SizeType             ArrayMax;

public:
	void WriteMemoryImage(FMemoryImageWriter& Writer) const
	{
		if constexpr (::TAllocatorTraits<AllocatorType>::SupportsFreezeMemoryImage && THasTypeLayout<ElementType>::Value)
		{
			this->AllocatorInstance.WriteMemoryImage(Writer, StaticGetTypeLayoutDesc<ElementType>(), this->ArrayNum);
			Writer.WriteBytes(this->ArrayNum);
			Writer.WriteBytes(this->ArrayNum);
		}
		else
		{
			// Writing non-freezable TArray is only supported for 64-bit target for now
			// Would need complete layout macros for all allocator types in order to properly write (empty) 32bit versions
			check(Writer.Is64BitTarget());
			Writer.WriteBytes(TArray());
		}
	}

	void CopyUnfrozen(const FMemoryUnfreezeContent& Context, void* Dst) const
	{
		if constexpr (::TAllocatorTraits<AllocatorType>::SupportsFreezeMemoryImage && THasTypeLayout<ElementType>::Value)
		{
			TArray* DstArray = ::new(Dst) TArray();
			DstArray->SetNumZeroed(this->ArrayNum);
			this->AllocatorInstance.CopyUnfrozen(Context, StaticGetTypeLayoutDesc<ElementType>(), this->ArrayNum, DstArray->GetData());
		}
		else
		{
			::new(Dst) TArray();
		}
	}

	static void AppendHash(const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		if constexpr (::TAllocatorTraits<AllocatorType>::SupportsFreezeMemoryImage && THasTypeLayout<ElementType>::Value)
		{
			Freeze::AppendHash(StaticGetTypeLayoutDesc<ElementType>(), LayoutParams, Hasher);
		}
	}

	void ToString(const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext) const
	{
		if constexpr (::TAllocatorTraits<AllocatorType>::SupportsFreezeMemoryImage && THasTypeLayout<ElementType>::Value)
		{
			this->AllocatorInstance.ToString(StaticGetTypeLayoutDesc<ElementType>(), this->ArrayNum, this->ArrayMax, LayoutParams, OutContext);
		}
	}

	/**
	* Implicit heaps
	*/
public:
	/** 
	 * Builds an implicit heap from the array.
	 *
	 * @param Predicate Predicate class instance.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during heapification.
	 *        Therefore, your predicate will be passed references rather than pointers.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	template <class PREDICATE_CLASS>
	UE_NODEBUG UE_FORCEINLINE_HINT void Heapify(const PREDICATE_CLASS& Predicate)
	{
		TDereferenceWrapper<ElementType, PREDICATE_CLASS> PredicateWrapper(Predicate);
		Algo::Heapify(*this, PredicateWrapper);
	}

	/**
	 * Builds an implicit heap from the array. Assumes < operator is defined
	 * for the template type.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during heapification.
	 *        Therefore, your array will be heapified by the values being pointed to, rather than the pointers' values.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	UE_NODEBUG void Heapify()
	{
		Heapify(TLess<>());
	}

	/** 
	 * Adds a new element to the heap.
	 *
	 * @param InItem Item to be added.
	 * @param Predicate Predicate class instance.
	 * @return The index of the new element.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during heapification.
	 *        Therefore, your predicate will be passed references rather than pointers.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	template <class PREDICATE_CLASS>
	SizeType HeapPush(ElementType&& InItem, const PREDICATE_CLASS& Predicate)
	{
		// Add at the end, then sift up
		Add(MoveTempIfPossible(InItem));
		TDereferenceWrapper<ElementType, PREDICATE_CLASS> PredicateWrapper(Predicate);
		SizeType Result = AlgoImpl::HeapSiftUp(GetData(), (SizeType)0, Num() - 1, FIdentityFunctor(), PredicateWrapper);

		return Result;
	}

	/** 
	 * Adds a new element to the heap.
	 *
	 * @param InItem Item to be added.
	 * @param Predicate Predicate class instance.
	 * @return The index of the new element.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during heapification.
	 *        Therefore, your predicate will be passed references rather than pointers.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	template <class PREDICATE_CLASS>
	SizeType HeapPush(const ElementType& InItem, const PREDICATE_CLASS& Predicate)
	{
		// Add at the end, then sift up
		Add(InItem);
		TDereferenceWrapper<ElementType, PREDICATE_CLASS> PredicateWrapper(Predicate);
		SizeType Result = AlgoImpl::HeapSiftUp(GetData(), (SizeType)0, Num() - 1, FIdentityFunctor(), PredicateWrapper);

		return Result;
	}

	/** 
	 * Adds a new element to the heap. Assumes < operator is defined for the
	 * template type.
	 *
	 * @param InItem Item to be added.
	 * @return The index of the new element.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during heapification.
	 *        Therefore, your array will be heapified by the values being pointed to, rather than the pointers' values.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	UE_NODEBUG SizeType HeapPush(ElementType&& InItem)
	{
		return HeapPush(MoveTempIfPossible(InItem), TLess<>());
	}

	/** 
	 * Adds a new element to the heap. Assumes < operator is defined for the
	 * template type.
	 *
	 * @param InItem Item to be added.
	 * @return The index of the new element.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during heapification.
	 *        Therefore, your array will be heapified by the values being pointed to, rather than the pointers' values.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	UE_NODEBUG SizeType HeapPush(const ElementType& InItem)
	{
		return HeapPush(InItem, TLess<>());
	}

	/** 
	 * Removes the top element from the heap.
	 *
	 * @param OutItem The removed item.
	 * @param Predicate Predicate class instance.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during heapification.
	 *        Therefore, your predicate will be passed references rather than pointers.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	template <class PREDICATE_CLASS>
	void HeapPop(ElementType& OutItem, const PREDICATE_CLASS& Predicate, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		OutItem = MoveTemp((*this)[0]);
		RemoveAtSwap(0, 1, AllowShrinking);

		TDereferenceWrapper< ElementType, PREDICATE_CLASS> PredicateWrapper(Predicate);
		AlgoImpl::HeapSiftDown(GetData(), (SizeType)0, Num(), FIdentityFunctor(), PredicateWrapper);
	}

	/** 
	 * Removes the top element from the heap. Assumes < operator is defined for
	 * the template type.
	 *
	 * @param OutItem The removed item.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during heapification.
	 *        Therefore, your array will be heapified by the values being pointed to, rather than the pointers' values.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	UE_NODEBUG void HeapPop(ElementType& OutItem, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		HeapPop(OutItem, TLess<>(), AllowShrinking);
	}

	/**
	 * Verifies the heap.
	 *
	 * @param Predicate Predicate class instance.
	 */
	template <class PREDICATE_CLASS>
	UE_NODEBUG void VerifyHeap(const PREDICATE_CLASS& Predicate)
	{
		check(Algo::IsHeap(*this, Predicate));
	}

	/** 
	 * Removes the top element from the heap.
	 *
	 * @param Predicate Predicate class instance.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during heapification.
	 *        Therefore, your predicate will be passed references rather than pointers.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	template <class PREDICATE_CLASS>
	void HeapPopDiscard(const PREDICATE_CLASS& Predicate, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		RemoveAtSwap(0, 1, AllowShrinking);
		TDereferenceWrapper< ElementType, PREDICATE_CLASS> PredicateWrapper(Predicate);
		AlgoImpl::HeapSiftDown(GetData(), (SizeType)0, Num(), FIdentityFunctor(), PredicateWrapper);
	}

	/** 
	 * Removes the top element from the heap. Assumes < operator is defined for the template type.
	 *
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during heapification.
	 *        Therefore, your array will be heapified by the values being pointed to, rather than the pointers' values.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	UE_NODEBUG void HeapPopDiscard(EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		HeapPopDiscard(TLess<>(), AllowShrinking);
	}

	/** 
	 * Returns the top element from the heap (does not remove the element).
	 *
	 * Const version.
	 *
	 * @returns The reference to the top element from the heap.
	 */
	UE_NODEBUG [[nodiscard]] const ElementType& HeapTop() const UE_LIFETIMEBOUND
	{
		return (*this)[0];
	}

	/** 
	 * Returns the top element from the heap (does not remove the element).
	 *
	 * @returns The reference to the top element from the heap.
	 */
	UE_NODEBUG [[nodiscard]] ElementType& HeapTop() UE_LIFETIMEBOUND
	{
		return (*this)[0];
	}

	/**
	 * Removes an element from the heap.
	 *
	 * @param Index Position at which to remove item.
	 * @param Predicate Predicate class instance.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during heapification.
	 *        Therefore, your predicate will be passed references rather than pointers.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	template <class PREDICATE_CLASS>
	void HeapRemoveAt(SizeType Index, const PREDICATE_CLASS& Predicate, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		RemoveAtSwap(Index, 1, AllowShrinking);

		TDereferenceWrapper< ElementType, PREDICATE_CLASS> PredicateWrapper(Predicate);
		AlgoImpl::HeapSiftDown(GetData(), Index, Num(), FIdentityFunctor(), PredicateWrapper);
		AlgoImpl::HeapSiftUp(GetData(), (SizeType)0, FPlatformMath::Min(Index, Num() - 1), FIdentityFunctor(), PredicateWrapper);
	}

	/**
	 * Removes an element from the heap. Assumes < operator is defined for the template type.
	 *
	 * @param Index Position at which to remove item.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during heapification.
	 *        Therefore, your array will be heapified by the values being pointed to, rather than the pointers' values.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	UE_NODEBUG void HeapRemoveAt(SizeType Index, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		HeapRemoveAt(Index, TLess<>(), AllowShrinking);
	}

	/**
	 * Performs heap sort on the array.
	 *
	 * @param Predicate Predicate class instance.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during heapification.
	 *        Therefore, your predicate will be passed references rather than pointers.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	template <class PREDICATE_CLASS>
	UE_NODEBUG void HeapSort(const PREDICATE_CLASS& Predicate)
	{
		TDereferenceWrapper<ElementType, PREDICATE_CLASS> PredicateWrapper(Predicate);
		Algo::HeapSort(*this, PredicateWrapper);
	}

	/**
	 * Performs heap sort on the array. Assumes < operator is defined for the
	 * template type.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during heapification.
	 *        Therefore, your array will be heapified by the values being pointed to, rather than the pointers' values.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	UE_NODEBUG void HeapSort()
	{
		HeapSort(TLess<>());
	}

	UE_NODEBUG [[nodiscard]] const ElementAllocatorType& GetAllocatorInstance() const
	{
		return AllocatorInstance;
	}
	UE_NODEBUG [[nodiscard]] ElementAllocatorType& GetAllocatorInstance()
	{
		return AllocatorInstance;
	}

	friend struct TArrayPrivateFriend;
};


namespace Freeze
{
	template<typename T, typename AllocatorType>
	UE_NODEBUG void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const TArray<T, AllocatorType>& Object, const FTypeLayoutDesc&)
	{
		Object.WriteMemoryImage(Writer);
	}

	template<typename T, typename AllocatorType>
	UE_NODEBUG [[nodiscard]] uint32 IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const TArray<T, AllocatorType>& Object, void* OutDst)
	{
		Object.CopyUnfrozen(Context, OutDst);
		return sizeof(Object);
	}

	template<typename T, typename AllocatorType>
	UE_NODEBUG [[nodiscard]] uint32 IntrinsicAppendHash(const TArray<T, AllocatorType>* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		return AppendHashForNameAndSize(TypeDesc.Name, sizeof(TArray<T, AllocatorType>), Hasher);
	}

	template<typename T, typename AllocatorType>
	UE_NODEBUG [[nodiscard]] uint32 IntrinsicGetTargetAlignment(const TArray<T, AllocatorType>* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams)
	{
		// Assume alignment of array is drive by pointer
		return FMath::Min(8u, LayoutParams.MaxFieldAlignment);
	}

	template<typename T, typename AllocatorType>
	UE_NODEBUG void IntrinsicToString(const TArray<T, AllocatorType>& Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
	{
		Object.ToString(LayoutParams, OutContext);
	}
}

DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT((template <typename T, typename AllocatorType>), (TArray<T, AllocatorType>));

template <typename InElementType, typename AllocatorType>
struct TIsZeroConstructType<TArray<InElementType, AllocatorType>>
{
	static constexpr bool Value = ::TAllocatorTraits<AllocatorType>::IsZeroConstruct;
};

template <typename T, typename AllocatorType>
struct TIsContiguousContainer<TArray<T, AllocatorType>>
{
	static constexpr bool Value = true;
};

/**
 * Trait which determines whether or not a type is a TArray.
 */
template <typename T> constexpr bool TIsTArray_V = false;

template <typename InElementType, typename InAllocatorType> constexpr bool TIsTArray_V<               TArray<InElementType, InAllocatorType>> = true;
template <typename InElementType, typename InAllocatorType> constexpr bool TIsTArray_V<const          TArray<InElementType, InAllocatorType>> = true;
template <typename InElementType, typename InAllocatorType> constexpr bool TIsTArray_V<      volatile TArray<InElementType, InAllocatorType>> = true;
template <typename InElementType, typename InAllocatorType> constexpr bool TIsTArray_V<const volatile TArray<InElementType, InAllocatorType>> = true;

template <typename T>
struct TIsTArray
{
	static constexpr bool Value = TIsTArray_V<T>;
};


//
// Array operator news.
//
template <typename T,typename AllocatorType>
UE_NODEBUG void* operator new(size_t Size, TArray<T, AllocatorType>& Array)
{
	check(Size == sizeof(T));
	const auto Index = Array.AddUninitialized();
	return &Array[Index];
}
template <typename T,typename AllocatorType>
UE_NODEBUG void* operator new(size_t Size, TArray<T,AllocatorType>& Array, typename TArray<T, AllocatorType>::SizeType Index)
{
	check(Size == sizeof(T));
	Array.InsertUninitialized(Index);
	return &Array[Index];
}

struct TArrayPrivateFriend
{
	/**
	 * Serialization operator.
	 *
	 * @param Ar Archive to serialize the array with.
	 * @param A Array to serialize.
	 * @returns Passing the given archive.
	 */
	template<typename ElementType, typename AllocatorType>
	static FArchive& Serialize(FArchive& Ar, TArray<ElementType, AllocatorType>& A)
	{
		A.CountBytes(Ar);

		// For net archives, limit serialization to 16MB, to protect against excessive allocation
		using SizeType = typename AllocatorType::SizeType;
		constexpr SizeType MaxNetArraySerialize = (16 * 1024 * 1024) / sizeof(ElementType);
		SizeType SerializeNum = Ar.IsLoading() ? 0 : A.ArrayNum;

		Ar << SerializeNum;

		if (SerializeNum == 0)
		{
			// if we are loading, then we have to reset the size to 0, in case it isn't currently 0
			if (Ar.IsLoading())
			{
				A.Empty();
			}
			return Ar;
		}

		if (Ar.IsError() || SerializeNum < 0 || !ensure(!Ar.IsNetArchive() || SerializeNum <= MaxNetArraySerialize))
		{
			Ar.SetError();
			return Ar;
		}

		// if we don't need to perform per-item serialization, just read it in bulk
		if constexpr (sizeof(ElementType) == 1 || TCanBulkSerialize<ElementType>::Value)
		{
			if (Ar.IsLoading())
			{
				A.ArrayNum = SerializeNum;
			}

			// Serialize simple bytes which require no construction or destruction.
			if ((A.ArrayNum || A.ArrayMax) && Ar.IsLoading())
			{
				UE::Core::Private::ReallocForCopy<UE::Core::Private::GetAllocatorFlags<AllocatorType>()>(
					sizeof(ElementType),
					alignof(ElementType),
					A.ArrayNum,
					A.ArrayMax,
					A.AllocatorInstance,
					A.ArrayNum,
					A.ArrayMax
				);
			}

			if constexpr (TIsUECoreVariant<ElementType, double>::Value)
			{
				if (Ar.IsLoading() && Ar.UEVer() < EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
				{
					// Per item serialization is required for core variant types loaded from pre LWC archives, to enable conversion from float to double.
					A.Empty(SerializeNum);
					for (SizeType i = 0; i < SerializeNum; i++)
					{
						Ar << A.AddDefaulted_GetRef();
					}
				}
				else
				{
					Ar.Serialize(A.GetData(), A.Num() * sizeof(ElementType));
				}
			}
			else
			{
				Ar.Serialize(A.GetData(), A.Num() * sizeof(ElementType));
			}

		}
		else if (Ar.IsLoading())
		{
			// Required for resetting ArrayNum
			A.Empty(SerializeNum);

			for (SizeType i=0; i<SerializeNum; i++)
			{
				Ar << A.AddDefaulted_GetRef();
			}
		}
		else
		{
			for (SizeType i=0; i<A.ArrayNum; i++)
			{
				Ar << A[i];
			}
		}

		A.SlackTrackerNumChanged();

		return Ar;
	}
};


template<typename ElementType, typename AllocatorType>
UE_NODEBUG FArchive& operator<<(FArchive& Ar, TArray<ElementType, AllocatorType>& A)
{
	return TArrayPrivateFriend::Serialize(Ar, A);
}

/** Returns a unique hash by combining those of each array element. */
template<typename InElementType, typename InAllocatorType>
[[nodiscard]] uint32 GetTypeHash(const TArray<InElementType, InAllocatorType>& A)
{
	uint32 Hash = 0;
	for (const InElementType& V : A)
	{
		Hash = HashCombineFast(Hash, GetTypeHash(V));
	}
	return Hash;
}
