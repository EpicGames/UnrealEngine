// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Array.h"

/**
 * Template for fluent array builders.
 *
 * @param ElementType The type of elements stored in the array.
 * @param Allocator The allocator to use for elements.
 */
template<typename ElementType, typename Allocator = FDefaultAllocator> class TArrayBuilder
{
public:

	/** Default constructor. */
	TArrayBuilder() { }

	/**
	 * Creates and initializes an array builder from an array of items.
	 *
	 * @param InArray The array of items to copy.
	 */
	template<typename OtherAllocator> TArrayBuilder( const TArray<ElementType, OtherAllocator>& InArray )
		: Array(InArray)
	{ }

public:

	/**
	 * Adds an item.
	 *
	 * @param Item The item to add.
	 * @return This instance (for method chaining).
	 * @see AddUnique
	 */
	TArrayBuilder& Add( const ElementType& Item )
	{
		Array.Add(Item);

		return *this;
	}

	/**
	 * Adds an unique item.
	 *
	 * @param Item The unique item to add.
	 * @return This instance (for method chaining).
	 * @see Add
	 */
	TArrayBuilder& AddUnique( const ElementType& Item )
	{
		Array.AddUnique(Item);

		return *this;
	}

	/**
	 * Appends an array of items.
	 *
	 * @param OtherArray The array to append.
	 * @return This instance (for method chaining).
	 */
	template<typename OtherAllocator> TArrayBuilder& Append( const TArray<ElementType, OtherAllocator>& OtherArray )
	{
		Array.Append(OtherArray);

		return *this;
	}

public:

	/**
	 * Builds the array as configured.
	 *
	 * @return A new array.
	 */
	TArray<ElementType, Allocator> Build()
	{
		return Array;
	}

	/**
	 * Implicit conversion operator to build the array as configured.
	 *
	 * @return A new array.
	 */
	operator TArray<ElementType, Allocator>()
	{
		return Build();
	}

private:

	/** Holds the array. */
	TArray<ElementType, Allocator> Array;
};
