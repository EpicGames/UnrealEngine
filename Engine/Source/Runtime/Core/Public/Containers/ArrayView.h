// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Concepts/Const.h"
#include "Containers/ContainersFwd.h" // IWYU pragma: export
#include "Misc/AssertionMacros.h"
#include "Misc/ReverseIterate.h"
#include "Misc/IntrusiveUnsetOptionalState.h"
#include "Misc/UEOps.h"
#include "Templates/Invoke.h"
#include "Templates/UnrealTypeTraits.h"
#include "Traits/ElementType.h"
#include "Containers/Array.h"
#include "Math/UnrealMathUtility.h"
#include <type_traits>

namespace UE::Core::ArrayView::Private
{
	/**
	 * Trait testing whether a type is compatible with the view type
	 *
	 * The extra stars here are *IMPORTANT*
	 * They prevent TMultiArrayView<Base>(TArray<Derived>&) from compiling!
	 */
	template <typename T, typename ElementType>
	constexpr bool TIsCompatibleElementType_V =std::is_convertible_v<T**, ElementType* const*>;

	// Gets the data from the passed argument and proceeds to reinterpret the resulting elements
	template <typename T>
	inline decltype(auto) GetReinterpretedDataHelper(T&& Arg)
	{
		auto NaturalPtr = GetData(Forward<T>(Arg));
		using NaturalElementType = std::remove_pointer_t<decltype(NaturalPtr)>;

		auto Size = GetNum(Arg);
		auto EndPtr = NaturalPtr + Size;
		TContainerElementTypeCompatibility<NaturalElementType>::ReinterpretRangeContiguous(NaturalPtr, EndPtr, Size);

		return reinterpret_cast<typename TContainerElementTypeCompatibility<NaturalElementType>::ReinterpretType*>(NaturalPtr);
	}

	/**
	 * Trait testing whether a type is compatible with the view type
	 */
	template <typename RangeType, typename ElementType>
	struct TIsCompatibleRangeType
	{
		static constexpr bool Value = TIsCompatibleElementType_V<std::remove_pointer_t<decltype(GetData(DeclVal<RangeType&>()))>, ElementType>;

		template <typename T>
		static constexpr decltype(auto) GetData(T&& Arg)
		{
			return GetDataHelper(Forward<T>(Arg));
		}
	};

	/**
	 * Trait testing whether a type is reinterpretable in a way that permits use with the view type
	 */
	template <typename RangeType, typename ElementType>
	struct TIsReinterpretableRangeType
	{
	private:
		using NaturalElementType = std::remove_pointer_t<decltype(GetData(DeclVal<RangeType&>()))>;
		using TypeCompat = TContainerElementTypeCompatibility<NaturalElementType>;

	public:
		static constexpr bool Value = 
			!std::is_same_v<typename TypeCompat::ReinterpretType, NaturalElementType>
			&&
			TIsCompatibleElementType_V<typename TypeCompat::ReinterpretType, ElementType>
			&&
			(!UE_DEPRECATE_MUTABLE_TOBJECTPTR
			 || std::is_same_v<ElementType, std::remove_pointer_t<typename TypeCompat::ReinterpretType>* const>
			 || std::is_same_v<ElementType, const std::remove_pointer_t<typename TypeCompat::ReinterpretType>* const>);

		template <typename T>
		static decltype(auto) GetData(T&& Arg)
		{
			return UE::Core::ArrayView::Private::GetReinterpretedDataHelper(Forward<T>(Arg));
		}
	};
}

template <typename T>                                  constexpr bool TIsTArrayView_V                                                       = false;
template <typename InElementType, typename InSizeType> constexpr bool TIsTArrayView_V<               TArrayView<InElementType, InSizeType>> = true;
template <typename InElementType, typename InSizeType> constexpr bool TIsTArrayView_V<      volatile TArrayView<InElementType, InSizeType>> = true;
template <typename InElementType, typename InSizeType> constexpr bool TIsTArrayView_V<const          TArrayView<InElementType, InSizeType>> = true;
template <typename InElementType, typename InSizeType> constexpr bool TIsTArrayView_V<const volatile TArrayView<InElementType, InSizeType>> = true;

template <typename T>
struct TIsTArrayView
{
	static constexpr bool Value = TIsTArrayView_V<T>;
	static constexpr bool value = TIsTArrayView_V<T>;
};

/**
 * Templated fixed-size view of another array
 *
 * A statically sized view of an array of typed elements.  Designed to allow functions to take either a fixed C array
 * or a TArray with an arbitrary allocator as an argument when the function neither adds nor removes elements
 *
 * e.g.:
 * int32 SumAll(TArrayView<const int32> array)
 * {
 *     return Algo::Accumulate(array);
 * }
 * 
 * could be called as:
 *     SumAll(MyTArray);\
 *     SumAll(MyCArray);
 *     SumAll(MakeArrayView(Ptr, Num));
 *
 *     auto Values = { 1, 2, 3 };
 *     SumAll(Values);
 *
 * Note:
 *   View classes are not const-propagating! If you want a view where the elements are const, you need "TArrayView<const T>" not "const TArrayView<T>"!
 *
 * Caution:
 *   Treat a view like a *reference* to the elements in the array. DO NOT free or reallocate the array while the view exists!
 *   For this reason, be mindful of lifetimes when constructing TArrayViews from rvalue initializer lists:
 *
 *   TArrayView<int> View = { 1, 2, 3 }; // construction of array view from rvalue initializer list
 *   int n = View[0]; // undefined behavior, as the initializer list was destroyed at the end of the previous line
 */
template<typename InElementType, typename InSizeType>
class TArrayView
{
public:
	using ElementType = InElementType;
	using SizeType = InSizeType;

	static_assert(std::is_signed_v<SizeType>, "TArrayView only supports signed index types");

	// Defaulted object behavior - we want compiler-generated functions rather than going through the generic range constructor.
	[[nodiscard]] constexpr TArrayView(const TArrayView&) = default;
	TArrayView& operator=(const TArrayView&) = default;
	~TArrayView() = default;

	/**
	 * Constructor.
	 */
	[[nodiscard]] constexpr TArrayView()
		: DataPtr(nullptr)
		, ArrayNum(0)
	{
	}

private:
	template <typename T>
	using TIsCompatibleRangeType = UE::Core::ArrayView::Private::TIsCompatibleRangeType<T, ElementType>;

	template <typename T>
	using TIsReinterpretableRangeType = UE::Core::ArrayView::Private::TIsReinterpretableRangeType<T, ElementType>;

public:
	/**
	 * Constructor from another range
	 *
	 * @param Other The source range to copy
	 */
	template <
		typename OtherRangeType,
		typename CVUnqualifiedOtherRangeType = std::remove_cv_t<std::remove_reference_t<OtherRangeType>>
		UE_REQUIRES(
			TAnd<
				TIsContiguousContainer<CVUnqualifiedOtherRangeType>,
				TOr<
					TIsCompatibleRangeType<OtherRangeType>,
					TIsReinterpretableRangeType<OtherRangeType>
				>
			>::Value &&
			TIsTArrayView_V<CVUnqualifiedOtherRangeType> &&
			!std::is_same_v<CVUnqualifiedOtherRangeType, TArrayView>
		)
	>
	[[nodiscard]] constexpr TArrayView(OtherRangeType&& Other)
		: DataPtr(std::conditional_t<
						TIsCompatibleRangeType<OtherRangeType>::Value,
						TIsCompatibleRangeType<OtherRangeType>,
						TIsReinterpretableRangeType<OtherRangeType>
					>::GetData(Forward<OtherRangeType>(Other)))
	{
		const auto InCount = GetNum(Forward<OtherRangeType>(Other));
		using InCountType = decltype(InCount);

		// Unlike the other constructor, we don't need to check(InCount >= 0), because it's coming from a TArrayView which guarantees that
		if constexpr (sizeof(InCountType) > sizeof(SizeType) || (sizeof(InCountType) == sizeof(SizeType) && std::is_unsigned_v<InCountType>))
		{
			check(InCount <= static_cast<InCountType>(TNumericLimits<SizeType>::Max()));
		}

		ArrayNum = (SizeType)InCount;
	}
	template <
		typename OtherRangeType,
		typename CVUnqualifiedOtherRangeType = std::remove_cv_t<std::remove_reference_t<OtherRangeType>>
		UE_REQUIRES(
			TAnd<
				TIsContiguousContainer<CVUnqualifiedOtherRangeType>,
				TOr<
					TIsCompatibleRangeType<OtherRangeType>,
					TIsReinterpretableRangeType<OtherRangeType>
				>
			>::Value &&
			!TIsTArrayView_V<CVUnqualifiedOtherRangeType>
		)
	>
	[[nodiscard]] constexpr TArrayView(OtherRangeType&& Other UE_LIFETIMEBOUND)
		: DataPtr(std::conditional_t<
			TIsCompatibleRangeType<OtherRangeType>::Value,
			TIsCompatibleRangeType<OtherRangeType>,
			TIsReinterpretableRangeType<OtherRangeType>
		>::GetData(Forward<OtherRangeType>(Other)))
	{
		const auto InCount = GetNum(Forward<OtherRangeType>(Other));
		using InCountType = decltype(InCount);
		if constexpr (sizeof(InCountType) > sizeof(SizeType) || (sizeof(InCountType) == sizeof(SizeType) && std::is_unsigned_v<InCountType>))
		{
			check(InCount >= 0 && InCount <= static_cast<InCountType>(TNumericLimits<SizeType>::Max()));
		}
		else
		{
			check(InCount >= 0);
		}
		ArrayNum = (SizeType)InCount;
	}

	/**
	 * Construct a view of an arbitrary pointer
	 *
	 * @param InData	The data to view
	 * @param InCount	The number of elements
	 */
	template <
		typename OtherElementType
		UE_REQUIRES(UE::Core::ArrayView::Private::TIsCompatibleElementType_V<OtherElementType, ElementType>)
	>
	[[nodiscard]] constexpr TArrayView(OtherElementType* InData UE_LIFETIMEBOUND, SizeType InCount)
		: DataPtr(InData)
		, ArrayNum(InCount)
	{
		check(ArrayNum >= 0);
	}

	/**
	 * Construct a view of an initializer list.
	 *
	 * The caller is responsible for ensuring that the view does not outlive the initializer list.
	 */
	[[nodiscard]] constexpr TArrayView(std::initializer_list<ElementType> List UE_LIFETIMEBOUND)
		requires UE::CConst<ElementType>
		: DataPtr(GetDataHelper(List))
		, ArrayNum(GetNum(List))
	{
	}

	//////////////////////////////////
	// Start - assignment operators //
	//////////////////////////////////

	/**
	 * We need to explicitly implement assignment operators, even though they're equivalent to copying an implicitly-
	 * constructed temporary, because UE_LIFETIMEBOUND isn't transitive across implicit construction.
	 */

	 /**
	 * Assign from another range
	 *
	 * @param Other The source range to copy
	 */
	template <
		typename OtherRangeType,
		typename CVUnqualifiedOtherRangeType = std::remove_cv_t<std::remove_reference_t<OtherRangeType>>
		UE_REQUIRES(
			TAnd<
				TIsContiguousContainer<CVUnqualifiedOtherRangeType>,
				TOr<
					TIsCompatibleRangeType<OtherRangeType>,
					TIsReinterpretableRangeType<OtherRangeType>
				>
			>::Value &&
			TIsTArrayView_V<CVUnqualifiedOtherRangeType> &&
			!std::is_same_v<CVUnqualifiedOtherRangeType, TArrayView>
		)
	>
	UE_REWRITE constexpr TArrayView& operator=(OtherRangeType&& Other)
	{
		return *this = TArrayView(Forward<OtherRangeType>(Other));
	}
	template <
		typename OtherRangeType,
		typename CVUnqualifiedOtherRangeType = std::remove_cv_t<std::remove_reference_t<OtherRangeType>>
		UE_REQUIRES(
			TAnd<
				TIsContiguousContainer<CVUnqualifiedOtherRangeType>,
				TOr<
					TIsCompatibleRangeType<OtherRangeType>,
					TIsReinterpretableRangeType<OtherRangeType>
				>
			>::Value &&
			!TIsTArrayView_V<CVUnqualifiedOtherRangeType>
		)
	>
	UE_REWRITE constexpr TArrayView& operator=(OtherRangeType&& Other UE_LIFETIMEBOUND)
	{
		return *this = TArrayView(Forward<OtherRangeType>(Other));
	}

	/**
	 * Assign from an initializer list.
	 *
	 * The caller is responsible for ensuring that the view does not outlive the initializer list.
	 */
	UE_REWRITE constexpr TArrayView& operator=(std::initializer_list<ElementType> List UE_LIFETIMEBOUND)
		requires UE::CConst<ElementType>
	{
		return *this = TArrayView(List);
	}

	////////////////////////////////
	// End - assignment operators //
	////////////////////////////////


	///////////////////////////////////////////////////
	// Start - intrusive TOptional<TArrayView> state //
	///////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TArrayView;

	[[nodiscard]] explicit constexpr TArrayView(FIntrusiveUnsetOptionalState)
		: DataPtr(nullptr)
		, ArrayNum(-1)
	{
	}
	[[nodiscard]] constexpr bool UEOpEquals(FIntrusiveUnsetOptionalState) const
	{
		return ArrayNum == -1;
	}
	/////////////////////////////////////////////////
	// End - intrusive TOptional<TArrayView> state //
	/////////////////////////////////////////////////

public:

	/**
	 * Helper function for returning a typed pointer to the first array entry.
	 *
	 * @returns Pointer to first array entry.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr ElementType* GetData() const
	{
		return DataPtr;
	}

	/** 
	 * Helper function returning the size of the inner type.
	 *
	 * @returns Size in bytes of array type.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT static constexpr size_t GetTypeSize()
	{
		return sizeof(ElementType);
	}

	/** 
	 * Helper function returning the alignment of the inner type.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT static constexpr size_t GetTypeAlignment()
	{
		return alignof(ElementType);
	}

	/**
	 * Checks array invariants: if array size is greater than or equal to zero.
	 */
	UE_FORCEINLINE_HINT constexpr void CheckInvariants() const
	{
		checkSlow(ArrayNum >= 0);
	}

	/**
	 * Checks if index is in array range.
	 *
	 * @param Index Index to check.
	 */
	constexpr void RangeCheck(SizeType Index) const
	{
		CheckInvariants();

		checkf((Index >= 0) & (Index < ArrayNum),TEXT("Array index out of bounds: %lld from an array of size %lld"), (long long)Index, (long long)ArrayNum); // & for one branch
	}

	/**
	 * Checks if a slice range [Index, Index+InNum) is in array range.
	 * Length is 0 is allowed on empty arrays; Index must be 0 in that case.
	 *
	 * @param Index Starting index of the slice.
	 * @param InNum Length of the slice.
	 */
	constexpr void SliceRangeCheck(SizeType Index, SizeType InNum) const
	{
		checkf(Index >= 0, TEXT("Invalid index (%lld)"), (long long)Index);
		checkf(InNum >= 0, TEXT("Invalid count (%lld)"), (long long)InNum);
		checkf(Index + InNum <= ArrayNum, TEXT("Range (index: %lld, count: %lld) lies outside the view of %lld elements"), (long long)Index, (long long)InNum, (long long)ArrayNum);
	}

	/**
	 * Tests if index is valid, i.e. greater than or equal to zero, and less than the number of elements in the array.
	 *
	 * @param Index Index to test.
	 *
	 * @returns True if index is valid. False otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr bool IsValidIndex(SizeType Index) const
	{
		return (Index >= 0) && (Index < ArrayNum);
	}

	/**
	 * Returns true if the array view is empty and contains no elements. 
	 *
	 * @returns True if the array view is empty.
	 * @see Num
	 */
	[[nodiscard]] constexpr bool IsEmpty() const
	{
		return ArrayNum == 0;
	}

	/**
	 * Returns number of elements in array.
	 *
	 * @returns Number of elements in array.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr SizeType Num() const
	{
		return ArrayNum;
	}

	/** @returns Number of bytes used, excluding slack */
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr SIZE_T NumBytes() const
	{
		return static_cast<SIZE_T>(ArrayNum) * sizeof(ElementType);
	}

	/**
	 * Array bracket operator. Returns reference to element at given index.
	 *
	 * @returns Reference to indexed element.
	 */
	[[nodiscard]] constexpr ElementType& operator[](SizeType Index) const
	{
		RangeCheck(Index);
		return GetData()[Index];
	}

	/**
	 * Returns n-th last element from the array.
	 *
	 * @param IndexFromTheEnd (Optional) Index from the end of array.
	 *                        Default is 0.
	 *
	 * @returns Reference to n-th last element from the array.
	 */
	[[nodiscard]] constexpr ElementType& Last(SizeType IndexFromTheEnd = 0) const
	{
		RangeCheck(ArrayNum - IndexFromTheEnd - 1);
		return GetData()[ArrayNum - IndexFromTheEnd - 1];
	}

	/**
	 * Returns a sliced view
	 * This is similar to Mid(), but with a narrow contract, i.e. slicing outside of the range of the view is illegal.
	 *
	 * @param Index starting index of the new view
	 * @param InNum number of elements in the new view
	 * @returns Sliced view
	 *
	 * @see Mid
	 */
	[[nodiscard]] constexpr TArrayView Slice(SizeType Index, SizeType InNum) const
	{
		SliceRangeCheck(Index, InNum);
		return TArrayView(DataPtr + Index, InNum);
	}

	/** Returns the left-most part of the view by taking the given number of elements from the left. */
	[[nodiscard]] constexpr TArrayView Left(SizeType Count) const
	{
		return TArrayView(DataPtr, FMath::Clamp(Count, (SizeType)0, ArrayNum));
	}

	/** Returns the left-most part of the view by chopping the given number of elements from the right. */
	[[nodiscard]] constexpr TArrayView LeftChop(SizeType Count) const
	{
		return TArrayView(DataPtr, FMath::Clamp(ArrayNum - Count, (SizeType)0, ArrayNum));
	}

	/** Returns the right-most part of the view by taking the given number of elements from the right. */
	[[nodiscard]] constexpr TArrayView Right(SizeType Count) const
	{
		const SizeType OutLen = FMath::Clamp(Count, (SizeType)0, ArrayNum);
		return TArrayView(DataPtr + ArrayNum - OutLen, OutLen);
	}

	/** Returns the right-most part of the view by chopping the given number of elements from the left. */
	[[nodiscard]] constexpr TArrayView RightChop(SizeType Count) const
	{
		const SizeType OutLen = FMath::Clamp(ArrayNum - Count, (SizeType)0, ArrayNum);
		return TArrayView(DataPtr + ArrayNum - OutLen, OutLen);
	}

	/** Returns the middle part of the view by taking up to the given number of elements from the given position. */
	[[nodiscard]] constexpr TArrayView Mid(SizeType Index, SizeType Count = TNumericLimits<SizeType>::Max()) const
	{
		ElementType* const CurrentStart  = GetData();
		const SizeType     CurrentLength = Num();

		// Clamp minimum index at the start of the range, adjusting the length down if necessary
		const SizeType NegativeIndexOffset = (Index < 0) ? Index : 0;
		Count += NegativeIndexOffset;
		Index -= NegativeIndexOffset;

		// Clamp maximum index at the end of the range
		Index = (Index > CurrentLength) ? CurrentLength : Index;

		// Clamp count between 0 and the distance to the end of the range
		Count = FMath::Clamp(Count, (SizeType)0, (CurrentLength - Index));

		TArrayView Result = TArrayView(CurrentStart + Index, Count);
		return Result;
	}

	/** Modifies the view to be the given number of elements from the left. */
	constexpr void LeftInline(SizeType CharCount)
	{
		*this = Left(CharCount);
	}

	/** Modifies the view by chopping the given number of elements from the right. */
	constexpr void LeftChopInline(SizeType CharCount)
	{
		*this = LeftChop(CharCount);
	}

	/** Modifies the view to be the given number of elements from the right. */
	constexpr void RightInline(SizeType CharCount)
	{
		*this = Right(CharCount);
	}

	/** Modifies the view by chopping the given number of elements from the left. */
	constexpr void RightChopInline(SizeType CharCount)
	{
		*this = RightChop(CharCount);
	}

	/** Modifies the view to be the middle part by taking up to the given number of elements from the given position. */
	constexpr void MidInline(SizeType Position, SizeType CharCount = TNumericLimits<SizeType>::Max())
	{
		*this = Mid(Position, CharCount);
	}

	/**
	 * Finds element within the array.
	 *
	 * @param Item Item to look for.
	 * @param Index Output parameter. Found index.
	 *
	 * @returns True if found. False otherwise.
	 */
	[[nodiscard]] constexpr bool Find(const ElementType& Item, SizeType& Index) const
	{
		Index = this->Find(Item);
		return Index != static_cast<SizeType>(INDEX_NONE);
	}

	/**
	 * Finds element within the array.
	 *
	 * @param Item Item to look for.
	 *
	 * @returns Index of the found element. INDEX_NONE otherwise.
	 */
	[[nodiscard]] constexpr SizeType Find(const ElementType& Item) const
	{
		const ElementType* RESTRICT Start = GetData();
		for (const ElementType* RESTRICT Data = Start, *RESTRICT DataEnd = Data + ArrayNum; Data != DataEnd; ++Data)
		{
			if (*Data == Item)
			{
				return static_cast<SizeType>(Data - Start);
			}
		}
		return static_cast<SizeType>(INDEX_NONE);
	}

	/**
	 * Finds element within the array starting from the end.
	 *
	 * @param Item Item to look for.
	 * @param Index Output parameter. Found index.
	 *
	 * @returns True if found. False otherwise.
	 */
	[[nodiscard]] constexpr bool FindLast(const ElementType& Item, SizeType& Index) const
	{
		Index = this->FindLast(Item);
		return Index != static_cast<SizeType>(INDEX_NONE);
	}

	/**
	 * Finds element within the array starting from the end.
	 *
	 * @param Item Item to look for.
	 * @returns Index of the found element. INDEX_NONE otherwise.
	 */
	[[nodiscard]] constexpr SizeType FindLast(const ElementType& Item) const
	{
		for (const ElementType* RESTRICT Start = GetData(), *RESTRICT Data = Start + ArrayNum; Data != Start; )
		{
			--Data;
			if (*Data == Item)
			{
				return static_cast<SizeType>(Data - Start);
			}
		}
		return static_cast<SizeType>(INDEX_NONE);
	}

	/**
	 * Finds element within the array starting from StartIndex and going backwards. Uses predicate to match element.
	 *
	 * @param Pred Predicate taking array element and returns true if element matches search criteria, false otherwise.
	 * @param StartIndex Index of element from which to start searching.
	 *
	 * @returns Index of the found element. INDEX_NONE otherwise.
	 */
	template <typename Predicate>
	[[nodiscard]] constexpr SizeType FindLastByPredicate(Predicate Pred, SizeType StartIndex) const
	{
		check(StartIndex >= 0 && StartIndex <= this->Num());
		for (const ElementType* RESTRICT Start = GetData(), *RESTRICT Data = Start + StartIndex; Data != Start; )
		{
			--Data;
			if (::Invoke(Pred, *Data))
			{
				return static_cast<SizeType>(Data - Start);
			}
		}
		return static_cast<SizeType>(INDEX_NONE);
	}

	/**
	* Finds element within the array starting from the end. Uses predicate to match element.
	*
	* @param Pred Predicate taking array element and returns true if element matches search criteria, false otherwise.
	*
	* @returns Index of the found element. INDEX_NONE otherwise.
	*/
	template <typename Predicate>
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr SizeType FindLastByPredicate(Predicate Pred) const
	{
		return FindLastByPredicate(Pred, ArrayNum);
	}

	/**
	 * Finds an item by key (assuming the ElementType overloads operator== for
	 * the comparison).
	 *
	 * @param Key The key to search by.
	 *
	 * @returns Index to the first matching element, or INDEX_NONE if none is
	 *          found.
	 */
	template <typename KeyType>
	[[nodiscard]] constexpr SizeType IndexOfByKey(const KeyType& Key) const
	{
		const ElementType* RESTRICT Start = GetData();
		for (const ElementType* RESTRICT Data = Start, *RESTRICT DataEnd = Start + ArrayNum; Data != DataEnd; ++Data)
		{
			if (*Data == Key)
			{
				return static_cast<SizeType>(Data - Start);
			}
		}
		return static_cast<SizeType>(INDEX_NONE);
	}

	/**
	 * Finds an item by predicate.
	 *
	 * @param Pred The predicate to match.
	 *
	 * @returns Index to the first matching element, or INDEX_NONE if none is
	 *          found.
	 */
	template <typename Predicate>
	[[nodiscard]] constexpr SizeType IndexOfByPredicate(Predicate Pred) const
	{
		const ElementType* RESTRICT Start = GetData();
		for (const ElementType* RESTRICT Data = Start, *RESTRICT DataEnd = Start + ArrayNum; Data != DataEnd; ++Data)
		{
			if (::Invoke(Pred, *Data))
			{
				return static_cast<SizeType>(Data - Start);
			}
		}
		return static_cast<SizeType>(INDEX_NONE);
	}

	/**
	 * Finds an item by key (assuming the ElementType overloads operator== for
	 * the comparison).
	 *
	 * @param Key The key to search by.
	 *
	 * @returns Pointer to the first matching element, or nullptr if none is found.
	 */
	template <typename KeyType>
	[[nodiscard]] constexpr ElementType* FindByKey(const KeyType& Key) const
	{
		for (ElementType* RESTRICT Data = GetData(), *RESTRICT DataEnd = Data + ArrayNum; Data != DataEnd; ++Data)
		{
			if (*Data == Key)
			{
				return Data;
			}
		}

		return nullptr;
	}

	/**
	 * Finds an element which matches a predicate functor.
	 *
	 * @param Pred The functor to apply to each element.
	 *
	 * @return Pointer to the first element for which the predicate returns
	 *         true, or nullptr if none is found.
	 */
	template <typename Predicate>
	[[nodiscard]] constexpr ElementType* FindByPredicate(Predicate Pred) const
	{
		for (ElementType* RESTRICT Data = GetData(), *RESTRICT DataEnd = Data + ArrayNum; Data != DataEnd; ++Data)
		{
			if (::Invoke(Pred, *Data))
			{
				return Data;
			}
		}

		return nullptr;
	}

	/**
	 * Filters the elements in the array based on a predicate functor.
	 *
	 * @param Pred The functor to apply to each element.
	 *
	 * @returns TArray with the same type as this object which contains
	 *          the subset of elements for which the functor returns true.
	 */
	template <typename Predicate>
	[[nodiscard]] constexpr TArray<std::remove_const_t<ElementType>> FilterByPredicate(Predicate Pred) const
	{
		TArray<std::remove_const_t<ElementType>> FilterResults;
		for (const ElementType* RESTRICT Data = GetData(), *RESTRICT DataEnd = Data + ArrayNum; Data != DataEnd; ++Data)
		{
			if (::Invoke(Pred, *Data))
			{
				FilterResults.Add(*Data);
			}
		}
		return FilterResults;
	}

	/**
	 * Checks if this array contains the element.
	 *
	 * @returns	True if found. False otherwise.
	 */
	template <typename ComparisonType>
	[[nodiscard]] constexpr bool Contains(const ComparisonType& Item) const
	{
		for (const ElementType* RESTRICT Data = GetData(), *RESTRICT DataEnd = Data + ArrayNum; Data != DataEnd; ++Data)
		{
			if (*Data == Item)
			{
				return true;
			}
		}
		return false;
	}

	/**
	 * Checks if this array contains an element for which the predicate is true.
	 *
	 * @param Predicate to use
	 *
	 * @returns	True if found. False otherwise.
	 */
	template <typename Predicate>
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr bool ContainsByPredicate(Predicate Pred) const
	{
		return FindByPredicate(Pred) != nullptr;
	}

public:
	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr ElementType*                         begin () const { return GetData(); }
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr ElementType*                         end   () const { return GetData() + Num(); }
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr TReversePointerIterator<ElementType> rbegin() const { return TReversePointerIterator<ElementType>(GetData() + Num()); }
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr TReversePointerIterator<ElementType> rend  () const { return TReversePointerIterator<ElementType>(GetData()); }

public:
	/**
	 * Sorts the array assuming < operator is defined for the item type.
	 *
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during sorting.
	 *        Therefore, your array will be sorted by the values being pointed to, rather than the pointers' values.
	 *        If this is not desirable, please use Algo::Sort(MyArray) directly instead.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	constexpr void Sort()
	{
		Algo::Sort(*this, TDereferenceWrapper<ElementType, TLess<>>(TLess<>()));
	}

	/**
	 * Sorts the array using user define predicate class.
	 *
	 * @param Predicate Predicate class instance.
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during sorting.
	 *        Therefore, your predicate will be passed references rather than pointers.
	 *        If this is not desirable, please use Algo::Sort(MyArray, Predicate) directly instead.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	template <class PREDICATE_CLASS>
	constexpr void Sort(const PREDICATE_CLASS& Predicate)
	{
		TDereferenceWrapper<ElementType, PREDICATE_CLASS> PredicateWrapper(Predicate);
		Algo::Sort(*this, PredicateWrapper);
	}

	/**
	 * Stable sorts the array assuming < operator is defined for the item type.
	 *
	 * Stable sort is slower than non-stable algorithm.
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during sorting.
	 *        Therefore, your array will be sorted by the values being pointed to, rather than the pointers' values.
	 *        If this is not desirable, please use Algo::StableSort(MyArray) directly instead.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	constexpr void StableSort()
	{
		Algo::StableSort(*this, TDereferenceWrapper<ElementType, TLess<>>(TLess<>()));
	}

	/**
	 * Stable sorts the array using user defined predicate class.
	 *
	 * Stable sort is slower than non-stable algorithm.
	 *
	 * @param Predicate Predicate class instance
	 * @note: If your array contains raw pointers, they will be automatically dereferenced during sorting.
	 *        Therefore, your predicate will be passed references rather than pointers.
	 *        If this is not desirable, please use Algo::StableSort(MyArray, Predicate) directly instead.
	 *        The auto-dereferencing behavior does not occur with smart pointers.
	 */
	template <class PREDICATE_CLASS>
	constexpr void StableSort(const PREDICATE_CLASS& Predicate)
	{
		TDereferenceWrapper<ElementType, PREDICATE_CLASS> PredicateWrapper(Predicate);
		Algo::StableSort(*this, PredicateWrapper);
	}

	// Comparison of array views to each other is not implemented because it
	// is not obvious whether the caller wants an exact match of the data
	// pointer and size, or to compare the objects being pointed to.
	template <typename OtherElementType, typename OtherSizeType>
	bool UEOpEquals(TArrayView<OtherElementType, OtherSizeType>) const = delete;

	/**
	 * Equality operator.
	 *
	 * @param Lhs Another ranged type to compare.
	 * @returns True if this array view's contents and the other ranged type match. False otherwise.
	 */
	template <typename RangeType>
		requires (std::is_convertible_v<decltype(GetDataHelper(std::declval<RangeType&>())), const ElementType*>)
	[[nodiscard]] bool UEOpEquals(RangeType&& Rhs) const
	{
		auto Count = GetNum(Rhs);
		return Count == ArrayNum && CompareItems(DataPtr, GetDataHelper(Rhs), Count);
	}

private:
	ElementType* DataPtr;
	SizeType ArrayNum;
};

template <typename T>
TArrayView(T*, int32) -> TArrayView<T>;

template <typename InElementType>
struct TIsZeroConstructType<TArrayView<InElementType>>
{
	enum { Value = true };
};

template <typename T, typename SizeType>
struct TIsContiguousContainer<TArrayView<T, SizeType>>
{
	enum { Value = true };
};

//////////////////////////////////////////////////////////////////////////

template <
	typename OtherRangeType,
	typename CVUnqualifiedOtherRangeType = std::remove_cv_t<std::remove_reference_t<OtherRangeType>>
	UE_REQUIRES(TIsContiguousContainer<CVUnqualifiedOtherRangeType>::Value && TIsTArrayView_V<CVUnqualifiedOtherRangeType>)
>
[[nodiscard]] constexpr auto MakeArrayView(OtherRangeType&& Other)
{
	return TArrayView<std::remove_pointer_t<decltype(GetData(DeclVal<OtherRangeType&>()))>>(Forward<OtherRangeType>(Other));
}

template <
	typename OtherRangeType,
	typename CVUnqualifiedOtherRangeType = std::remove_cv_t<std::remove_reference_t<OtherRangeType>>
	UE_REQUIRES(TIsContiguousContainer<CVUnqualifiedOtherRangeType>::Value && TIsTArrayView_V<CVUnqualifiedOtherRangeType>)
>
[[nodiscard]] constexpr auto MakeArrayView64(OtherRangeType&& Other)
{
	return TArrayView64<std::remove_pointer_t<decltype(GetData(DeclVal<OtherRangeType&>()))>>(Forward<OtherRangeType>(Other));
}

template <
	typename OtherRangeType,
	typename CVUnqualifiedOtherRangeType = std::remove_cv_t<std::remove_reference_t<OtherRangeType>>
	UE_REQUIRES(TIsContiguousContainer<CVUnqualifiedOtherRangeType>::Value && !TIsTArrayView_V<CVUnqualifiedOtherRangeType>)
>
[[nodiscard]] constexpr auto MakeArrayView(OtherRangeType&& Other UE_LIFETIMEBOUND)
{
	return TArrayView<std::remove_pointer_t<decltype(GetData(DeclVal<OtherRangeType&>()))>>(Forward<OtherRangeType>(Other));
}

template <
	typename OtherRangeType,
	typename CVUnqualifiedOtherRangeType = std::remove_cv_t<std::remove_reference_t<OtherRangeType>>
	UE_REQUIRES(TIsContiguousContainer<CVUnqualifiedOtherRangeType>::Value && !TIsTArrayView_V<CVUnqualifiedOtherRangeType>)
>
[[nodiscard]] constexpr auto MakeArrayView64(OtherRangeType&& Other UE_LIFETIMEBOUND)
{
	return TArrayView64<std::remove_pointer_t<decltype(GetData(DeclVal<OtherRangeType&>()))>>(Forward<OtherRangeType>(Other));
}

template<typename ElementType>
[[nodiscard]] constexpr TArrayView<ElementType> MakeArrayView(ElementType* Pointer UE_LIFETIMEBOUND, int32 Size)
{
	return TArrayView<ElementType>(Pointer, Size);
}

template<typename ElementType>
[[nodiscard]] constexpr TArrayView64<ElementType> MakeArrayView64(ElementType* Pointer UE_LIFETIMEBOUND, int64 Size)
{
	return TArrayView64<ElementType>(Pointer, Size);
}

template <typename ElementType>
[[nodiscard]] constexpr TArrayView<const ElementType> MakeArrayView(std::initializer_list<ElementType> List UE_LIFETIMEBOUND)
{
	return TArrayView<const ElementType>(List.begin(), List.size());
}

template <typename ElementType>
[[nodiscard]] constexpr TArrayView64<const ElementType> MakeArrayView64(std::initializer_list<ElementType> List UE_LIFETIMEBOUND)
{
	return TArrayView64<const ElementType>(List.begin(), List.size());
}

template <
	typename OtherRangeType,
	typename CVUnqualifiedOtherRangeType = std::remove_cv_t<std::remove_reference_t<OtherRangeType>>
	UE_REQUIRES(TIsContiguousContainer<CVUnqualifiedOtherRangeType>::Value && TIsTArrayView_V<CVUnqualifiedOtherRangeType>)
>
[[nodiscard]] constexpr auto MakeConstArrayView(OtherRangeType&& Other)
{
	return TArrayView<const std::remove_pointer_t<decltype(GetData(DeclVal<OtherRangeType&>()))>>(Forward<OtherRangeType>(Other));
}

template <
	typename OtherRangeType,
	typename CVUnqualifiedOtherRangeType = std::remove_cv_t<std::remove_reference_t<OtherRangeType>>
	UE_REQUIRES(TIsContiguousContainer<CVUnqualifiedOtherRangeType>::Value && TIsTArrayView_V<CVUnqualifiedOtherRangeType>)
>
[[nodiscard]] constexpr auto MakeConstArrayView64(OtherRangeType&& Other)
{
	return TArrayView64<const std::remove_pointer_t<decltype(GetData(DeclVal<OtherRangeType&>()))>>(Forward<OtherRangeType>(Other));
}

template <
	typename OtherRangeType,
	typename CVUnqualifiedOtherRangeType = std::remove_cv_t<std::remove_reference_t<OtherRangeType>>
	UE_REQUIRES(TIsContiguousContainer<CVUnqualifiedOtherRangeType>::Value && !TIsTArrayView_V<CVUnqualifiedOtherRangeType>)
>
[[nodiscard]] constexpr auto MakeConstArrayView(OtherRangeType&& Other UE_LIFETIMEBOUND)
{
	return TArrayView<const std::remove_pointer_t<decltype(GetData(DeclVal<OtherRangeType&>()))>>(Forward<OtherRangeType>(Other));
}

template <
	typename OtherRangeType,
	typename CVUnqualifiedOtherRangeType = std::remove_cv_t<std::remove_reference_t<OtherRangeType>>
	UE_REQUIRES(TIsContiguousContainer<CVUnqualifiedOtherRangeType>::Value && !TIsTArrayView_V<CVUnqualifiedOtherRangeType>)
>
[[nodiscard]] constexpr auto MakeConstArrayView64(OtherRangeType&& Other UE_LIFETIMEBOUND)
{
	return TArrayView64<const std::remove_pointer_t<decltype(GetData(DeclVal<OtherRangeType&>()))>>(Forward<OtherRangeType>(Other));
}

template<typename ElementType>
[[nodiscard]] constexpr TArrayView<const ElementType> MakeConstArrayView(const ElementType* Pointer UE_LIFETIMEBOUND, int32 Size)
{
	return TArrayView<const ElementType>(Pointer, Size);
}

template<typename ElementType>
[[nodiscard]] constexpr TArrayView64<const ElementType> MakeConstArrayView64(const ElementType* Pointer UE_LIFETIMEBOUND, int64 Size)
{
	return TArrayView64<const ElementType>(Pointer, Size);
}

template <typename ElementType>
[[nodiscard]] constexpr TArrayView<const ElementType> MakeConstArrayView(std::initializer_list<ElementType> List UE_LIFETIMEBOUND)
{
	return TArrayView<const ElementType>(List.begin(), List.size());
}

template <typename ElementType>
[[nodiscard]] constexpr TArrayView64<const ElementType> MakeConstArrayView64(std::initializer_list<ElementType> List UE_LIFETIMEBOUND)
{
	return TArrayView64<const ElementType>(List.begin(), List.size());
}

//////////////////////////////////////////////////////////////////////////

template<typename InElementType, typename InAllocatorType>
template<typename OtherElementType, typename OtherSizeType>
[[nodiscard]] UE_FORCEINLINE_HINT TArray<InElementType, InAllocatorType>::TArray(const TArrayView<OtherElementType, OtherSizeType>& Other)
{
	CopyToEmpty(Other.GetData(), Other.Num(), 0);
}

template<typename InElementType, typename InAllocatorType>
template<typename OtherElementType, typename OtherSizeType>
[[nodiscard]] UE_FORCEINLINE_HINT TArray<InElementType, InAllocatorType>::TArray(ElementAllocatorType&& InAllocatorInstance, const TArrayView<OtherElementType, OtherSizeType>& Other)
	: AllocatorInstance(MoveTemp(InAllocatorInstance))
{
	check(!AllocatorInstance.HasAllocation());
	CopyToEmpty(Other.GetData(), Other.Num(), 0);
}

template<typename InElementType, typename InAllocatorType>
template<typename OtherElementType, typename OtherSizeType>
inline TArray<InElementType, InAllocatorType>& TArray<InElementType, InAllocatorType>::operator=(const TArrayView<OtherElementType, OtherSizeType>& Other)
{
	DestructItems(GetData(), ArrayNum);
	CopyToEmpty(Other.GetData(), Other.Num(), ArrayMax);
	return *this;
}
