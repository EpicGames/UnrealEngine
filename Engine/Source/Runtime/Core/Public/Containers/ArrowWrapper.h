// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UnrealTemplate.h"

/**
 * Wrapper type for Iterators that return a structure by value from operator*, and want to support Iter->Property
 * to behave the same as (*Iter).Property.
 * When returning a structure by reference, operator* automatically handles -> as well, but when returning by value
 * it does not.
 * 
 * Example:
 * struct FIterator
 * {
 * public:
 *    ...
 *		// The problematic return-by-value structure from operator*
 *		// Without operator->, (*Iter).Key compiles but Iter->Key does not.
 *		TPair<FStringView, ViewedValueType&> operator*() const;
 * 
 *      // TArrowWrapper takes the value from operator* and allows -> to access it.
 *		TArrowWrapper<TPair<FStringView, ViewedValueType&>> operator->() const;
 * };
 */
template <typename WrappedType>
struct TArrowWrapper
{
	explicit TArrowWrapper(const WrappedType& InValue)
		: Value(InValue)
	{
	}

	explicit TArrowWrapper(WrappedType&& InValue)
		: Value(MoveTemp(InValue))
	{
	}

	const WrappedType* operator->() const
	{
		return &this->Value;
	}

	WrappedType Value;
};
