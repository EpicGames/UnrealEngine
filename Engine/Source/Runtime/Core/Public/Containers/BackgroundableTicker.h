// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/IDelegateInstance.h"
#include "Ticker.h"

/**
 * This works the same as the core FTSTicker, but on supported mobile platforms
 * it continues ticking while the app is running in the background.
 */
class FTSBackgroundableTicker
	: public FTSTicker
{
public:
	static CORE_API FTSBackgroundableTicker& GetCoreTicker();

	CORE_API FTSBackgroundableTicker();
	CORE_API ~FTSBackgroundableTicker();

private:
	FDelegateHandle CoreTickerHandle;
	::FDelegateHandle BackgroundTickerHandle;
	bool bWasBackgrounded = false;
};
