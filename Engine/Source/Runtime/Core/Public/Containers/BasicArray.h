// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not used?

#include "Templates/MemoryOps.h"
#include "Traits/IsContiguousContainer.h"

#if _MSC_VER
	#pragma warning(push)
	#pragma warning(disable : 4200)
#endif

/**
 * Basic RAII array which can be used without the excessive dependencies needed by TArray,
 * which needs to be serialisable, have a fixed ABI, have lots of helper algorithms as members,
 * have auto-shrinking, support allocators etc.
 */
template <typename T>
class TBasicArray
{
public:
	[[nodiscard]] TBasicArray()
		: Data(nullptr)
	{
	}

	// Non-copyable for now, but this could be made copyable in future if needed.
	TBasicArray(const TBasicArray&) = delete;
	TBasicArray& operator=(const TBasicArray&) = delete;

	[[nodiscard]] TBasicArray(TBasicArray&& Other)
		: Data(Other.Data)
	{
		Other.Data = nullptr;
	}

	TBasicArray& operator=(TBasicArray&& Other)
	{
		TBasicArray Temp(MoveTemp(Other));

		FData* TempData = Temp.Data;
		Temp.Data = this->Data;
		this->Data = TempData;

		return *this;
	}

	~TBasicArray()
	{
		if (FData* LocalData = this->Data)
		{
			DestructItems(LocalData->Data, LocalData->Num);
			FMemory::Free(LocalData);
		}
	}

	template <typename... ArgTypes>
	int32 Emplace(ArgTypes&&... Args)
	{
		int32 Result = Num();

		void* LocationToAdd = InsertUninitialized(Result);
		::new (LocationToAdd) T(Forward<ArgTypes>(Args)...);

		return Result;
	}

	template <typename... ArgTypes>
	void EmplaceAt(int32 Index, ArgTypes&&... Args)
	{
		void* LocationToAdd = InsertUninitialized(Index);
		::new (LocationToAdd) T(Forward<ArgTypes>(Args)...);
	}

	void RemoveAt(int32 Index, int32 NumToRemove = 1)
	{
		if (NumToRemove > 0)
		{
			FData* LocalData = this->Data;
			int32  LocalNum  = LocalData->Num;

			T* StartToRemove = &LocalData->Data[Index];
			DestructItems(StartToRemove, NumToRemove);
			RelocateConstructItems<T>(StartToRemove, StartToRemove + NumToRemove, LocalNum - NumToRemove);
			LocalData->Num -= NumToRemove;
		}
	}

	[[nodiscard]] bool IsEmpty() const
	{
		return Num() == 0;
	}

	[[nodiscard]] int32 Num() const
	{
		FData* LocalData = Data;
		return LocalData ? LocalData->Num : 0;
	}

	[[nodiscard]] T* GetData()
	{
		FData* LocalData = Data;
		return LocalData ? &LocalData->Data[0] : nullptr;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT const T* GetData() const
	{
		return const_cast<TBasicArray*>(this)->GetData();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT T& operator[](int32 Index)
	{
		return GetData()[Index];
	}

	[[nodiscard]] UE_FORCEINLINE_HINT const T& operator[](int32 Index) const
	{
		return GetData()[Index];
	}

private:
	static constexpr int32 InitialReservationSize = 16;

	static UE_FORCEINLINE_HINT int32 ApplyGrowthFactor(int32 CurrentNum)
	{
		return CurrentNum + CurrentNum / 2;
	}

	void* InsertUninitialized(int32 IndexToAdd)
	{
		FData* LocalData     = this->Data;
		T*     LocationToAdd = nullptr;
		if (!LocalData)
		{
			// IndexToAdd *must* be 0 here - can't assert that though

			LocalData = (FData*)FMemory::Malloc(sizeof(FData) + InitialReservationSize * sizeof(T));
			LocalData->Num = 1;
			LocalData->Max = InitialReservationSize;
			Data = LocalData;
			LocationToAdd = LocalData->Data;
		}
		else
		{
			int32 LocalNum = LocalData->Num;
			int32 LocalMax = LocalData->Max;

			if (LocalNum == LocalMax)
			{
				LocalMax = ApplyGrowthFactor(LocalMax);
				LocalData = (FData*)FMemory::Realloc(LocalData, sizeof(FData) + LocalMax * sizeof(T));
				LocalData->Max = LocalMax;
				this->Data = LocalData;
			}
			LocationToAdd = LocalData->Data + IndexToAdd;
			RelocateConstructItems<T>(LocationToAdd + 1, LocationToAdd, LocalNum - IndexToAdd);
			++LocalData->Num;
		}

		return LocationToAdd;
	}

	struct FData
	{
		int32 Num;
		int32 Max;
		T     Data[0];
	};

	FData* Data;

	[[nodiscard]] friend       T* begin(      TBasicArray& Arr) { return Arr.GetData(); }
	[[nodiscard]] friend const T* begin(const TBasicArray& Arr) { return Arr.GetData(); }
	[[nodiscard]] friend       T* end  (      TBasicArray& Arr) { return Arr.GetData() + Arr.Num(); }
	[[nodiscard]] friend const T* end  (const TBasicArray& Arr) { return Arr.GetData() + Arr.Num(); }
};

template <typename T>
struct TIsContiguousContainer<TBasicArray<T>>
{
	enum { Value = true };
};

#if _MSC_VER
	#pragma warning(pop)
#endif
