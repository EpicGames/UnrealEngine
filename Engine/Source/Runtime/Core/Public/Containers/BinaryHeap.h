// Copyright (C) 2009 Nine Realms, Inc
//

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"

#include <type_traits>

/*-----------------------------------------------------------------------------
	Binary Heap, used to index another data structure.

	Also known as a priority queue. Smallest key at top.
	KeyType must implement operator<
-----------------------------------------------------------------------------*/
template< typename KeyType, typename IndexType = uint32 >
class FBinaryHeap
{
public:
	static_assert(std::is_unsigned_v<IndexType>, "FBinaryHeap only supports unsigned index types");

				[[nodiscard]] FBinaryHeap();
				[[nodiscard]] FBinaryHeap( uint32 InHeapSize, uint32 InIndexSize );
				~FBinaryHeap();

				FBinaryHeap(const FBinaryHeap&) = delete;
				void operator =(const FBinaryHeap&) = delete;
				FBinaryHeap(FBinaryHeap&& Other);

	void		Clear();
	void		Free();
	void		Resize( uint32 NewHeapSize, uint32 NewIndexSize );

	[[nodiscard]] bool		IsEmpty() const			{ return HeapNum == 0; }
	[[nodiscard]] uint32		Num() const				{ return HeapNum; }
	[[nodiscard]] uint32		GetHeapSize() const		{ return HeapSize; }
	[[nodiscard]] uint32		GetIndexSize() const	{ return IndexSize; }

	[[nodiscard]] bool		IsPresent( IndexType Index ) const;
	[[nodiscard]] KeyType		GetKey( IndexType Index ) const;
	[[nodiscard]] IndexType	Peek( IndexType Index ) const;

	[[nodiscard]] IndexType	Top() const;
	void		Pop();

	void		Add( KeyType Key, IndexType Index );
	void		Update( KeyType Key, IndexType Index );
	void		Remove( IndexType Index );

protected:
	void		ResizeHeap( uint32 NewHeapSize );
	void		ResizeIndexes( uint32 NewIndexSize );

	void		UpHeap( IndexType HeapIndex );
	void		DownHeap( IndexType HeapIndex );

	/**
	 * Reset internal variables to a cleared state, does not free data.
	 */
	void		ResetInternal();

	uint32		HeapNum;
	uint32		HeapSize;
	uint32		IndexSize;
	
	IndexType*	Heap;

	KeyType*	Keys;
	IndexType*	HeapIndexes;
};

template< typename KeyType, typename IndexType >
[[nodiscard]] UE_FORCEINLINE_HINT FBinaryHeap< KeyType, IndexType >::FBinaryHeap()
{
	ResetInternal();
}

template< typename KeyType, typename IndexType >
[[nodiscard]] inline FBinaryHeap< KeyType, IndexType >::FBinaryHeap( uint32 InHeapSize, uint32 InIndexSize )
	: HeapNum(0)
	, HeapSize( InHeapSize )
	, IndexSize( InIndexSize )
{
	Heap = new IndexType[ HeapSize ];
	Keys = new KeyType[ IndexSize ];
	HeapIndexes = new IndexType[ IndexSize ];

	FMemory::Memset( HeapIndexes, 0xff, IndexSize * sizeof( IndexType ) );
}

template< typename KeyType, typename IndexType >
[[nodiscard]] inline FBinaryHeap< KeyType, IndexType >::FBinaryHeap(FBinaryHeap< KeyType, IndexType >&& Other)
{
	HeapNum = Other.HeapNum;
	HeapSize = Other.HeapSize;
	IndexSize = Other.IndexSize;

	Heap = Other.Heap;
	Keys = Other.Keys;
	HeapIndexes = Other.HeapIndexes;

	Other.ResetInternal();
}

template< typename KeyType, typename IndexType >
UE_FORCEINLINE_HINT FBinaryHeap< KeyType, IndexType >::~FBinaryHeap()
{
	Free();
}

template< typename KeyType, typename IndexType >
inline void FBinaryHeap< KeyType, IndexType >::Clear()
{
	HeapNum = 0;
	FMemory::Memset( HeapIndexes, 0xff, IndexSize * sizeof( IndexType ) );
}

template< typename KeyType, typename IndexType >
inline void FBinaryHeap< KeyType, IndexType >::Free()
{
	delete[] Heap;
	delete[] Keys;
	delete[] HeapIndexes;

	ResetInternal();
}

template< typename KeyType, typename IndexType >
void FBinaryHeap< KeyType, IndexType >::ResizeHeap( uint32 NewHeapSize )
{
	checkSlow( NewHeapSize != HeapSize );

	if( NewHeapSize == 0 )
	{
		HeapNum = 0;
		HeapSize = 0;
		
		delete[] Heap;
		Heap = nullptr;

		return;
	}

	IndexType* NewHeap = new IndexType[ NewHeapSize ];

	if( HeapSize != 0 )
	{
		FMemory::Memcpy( NewHeap, Heap, HeapSize * sizeof( IndexType ) );
		delete[] Heap;
	}
	
	HeapNum		= FMath::Min( HeapNum, NewHeapSize );
	HeapSize	= NewHeapSize;
	Heap		= NewHeap;
}

template< typename KeyType, typename IndexType >
void FBinaryHeap< KeyType, IndexType >::ResizeIndexes( uint32 NewIndexSize )
{
	checkSlow( NewIndexSize != IndexSize );

	if( NewIndexSize == 0 )
	{
		IndexSize = 0;
		
		delete[] Keys;
		delete[] HeapIndexes;

		Keys = nullptr;
		HeapIndexes = nullptr;

		return;
	}

	KeyType*	NewKeys			= new KeyType[ NewIndexSize ];
	IndexType*	NewHeapIndexes	= new IndexType[ NewIndexSize ];

	if( IndexSize != 0 )
	{
		check( NewIndexSize >= IndexSize );

		for( uint32 i = 0; i < IndexSize; i++ )
		{
			NewKeys[i] = Keys[i];
			NewHeapIndexes[i] = HeapIndexes[i];
		}
		delete[] Keys;
		delete[] HeapIndexes;
	}

	for( uint32 i = IndexSize; i < NewIndexSize; i++ )
	{
		NewHeapIndexes[i] = (IndexType)-1;
	}
	
	IndexSize	= NewIndexSize;
	Keys		= NewKeys;
	HeapIndexes	= NewHeapIndexes;
}

template< typename KeyType, typename IndexType >
inline void FBinaryHeap< KeyType, IndexType >::Resize( uint32 NewHeapSize, uint32 NewIndexSize )
{
	if( NewHeapSize != HeapSize )
	{
		ResizeHeap( NewHeapSize );
	}

	if( NewIndexSize != IndexSize )
	{
		ResizeIndexes( NewIndexSize );
	}
}

template< typename KeyType, typename IndexType >
[[nodiscard]] inline bool FBinaryHeap< KeyType, IndexType >::IsPresent( IndexType Index ) const
{
	if (Index >= IndexSize)
	{
		return false;
	}
	return HeapIndexes[ Index ] != (IndexType)-1;
}

template< typename KeyType, typename IndexType >
[[nodiscard]] inline KeyType FBinaryHeap< KeyType, IndexType >::GetKey( IndexType Index ) const
{
	checkSlow( IsPresent( Index ) );
	return Keys[ Index ];
}

template< typename KeyType, typename IndexType >
[[nodiscard]] inline IndexType FBinaryHeap< KeyType, IndexType >::Peek(IndexType Index) const
{
	checkSlow(Index < HeapNum);
	return Heap[Index];
}

template< typename KeyType, typename IndexType >
[[nodiscard]] inline IndexType FBinaryHeap< KeyType, IndexType >::Top() const
{
	checkSlow( Heap );
	checkSlow( HeapNum > 0 );
	return Heap[0];
}

template< typename KeyType, typename IndexType >
inline void FBinaryHeap< KeyType, IndexType >::Pop()
{
	checkSlow( Heap );
	checkSlow( HeapNum > 0 );

	IndexType Index = Heap[0];
	
	Heap[0] = Heap[ --HeapNum ];
	HeapIndexes[ Heap[0] ] = 0;
	HeapIndexes[ Index ] = (IndexType)-1;

	DownHeap(0);
}

template< typename KeyType, typename IndexType >
inline void FBinaryHeap< KeyType, IndexType >::Add( KeyType Key, IndexType Index )
{
	if( HeapNum == HeapSize )
	{
		ResizeHeap( FMath::Max<uint32>( 32u, HeapSize * 2 ) );
	}

	if( Index >= IndexSize )
	{
		ResizeIndexes(FMath::Max<uint32>(32u, FMath::RoundUpToPowerOfTwo(Index + 1)));
	}

	checkSlow( !IsPresent( Index ) );

	IndexType HeapIndex = HeapNum++;
	Heap[ HeapIndex ] = Index;
	
	Keys[ Index ] = Key;
	HeapIndexes[ Index ] = HeapIndex;

	UpHeap( HeapIndex );
}

template< typename KeyType, typename IndexType >
inline void FBinaryHeap< KeyType, IndexType >::Update( KeyType Key, IndexType Index )
{
	checkSlow( Heap );
	checkSlow( IsPresent( Index ) );

	Keys[ Index ] = Key;

	IndexType HeapIndex = HeapIndexes[ Index ];
	IndexType Parent = (HeapIndex - 1) >> 1;
	if( HeapIndex > 0 && Key < Keys[ Heap[ Parent ] ] )
	{
		UpHeap( HeapIndex );
	}
	else
	{
		DownHeap( HeapIndex );
	}
}

template< typename KeyType, typename IndexType >
inline void FBinaryHeap< KeyType, IndexType >::Remove( IndexType Index )
{
	if( !IsPresent( Index ) )
	{
		return;
	}

	KeyType Key = Keys[ Index ];
	IndexType HeapIndex = HeapIndexes[ Index ];

	Heap[ HeapIndex ] = Heap[ --HeapNum ];
	HeapIndexes[ Heap[ HeapIndex ] ] = HeapIndex;
	HeapIndexes[ Index ] = (IndexType)-1;

	if( Key < Keys[ Heap[ HeapIndex ] ] )
	{
		DownHeap( HeapIndex );
	}
	else
	{
		UpHeap( HeapIndex );
	}
}

template< typename KeyType, typename IndexType >
void FBinaryHeap< KeyType, IndexType >::UpHeap( IndexType HeapIndex )
{
	IndexType Moving = Heap[ HeapIndex ];
	IndexType i = HeapIndex;
	IndexType Parent = (i - 1) >> 1;

	while( i > 0 && Keys[ Moving ] < Keys[ Heap[ Parent ] ] )
	{
		Heap[i] = Heap[ Parent ];
		HeapIndexes[ Heap[i] ] = i;

		i = Parent;
		Parent = (i - 1) >> 1;
	}

	if( i != HeapIndex )
	{
		Heap[i] = Moving;
		HeapIndexes[ Heap[i] ] = i;
	}
}

template< typename KeyType, typename IndexType >
void FBinaryHeap< KeyType, IndexType >::DownHeap( IndexType HeapIndex )
{
	IndexType Moving = Heap[ HeapIndex ];
	IndexType i = HeapIndex;
	IndexType Left = (i << 1) + 1;
	IndexType Right = Left + 1;
	
	while( Left < HeapNum )
	{
		IndexType Smallest = Left;
		if( Right < HeapNum )
		{
			Smallest = ( Keys[ Heap[Left] ] < Keys[ Heap[Right] ] ) ? Left : Right;
		}

		if( Keys[ Heap[Smallest] ] < Keys[ Moving ] )
		{
			Heap[i] = Heap[ Smallest ];
			HeapIndexes[ Heap[i] ] = i;

			i = Smallest;
			Left = (i << 1) + 1;
			Right = Left + 1;
		}
		else
		{
			break;
		}
	}

	if( i != HeapIndex )
	{
		Heap[i] = Moving;
		HeapIndexes[ Heap[i] ] = i;
	}
}

template< typename KeyType, typename IndexType >
inline void FBinaryHeap< KeyType, IndexType >::ResetInternal()
{
	HeapNum = 0;
	HeapSize = 0;
	IndexSize = 0;
	Heap = nullptr;
	Keys = nullptr;
	HeapIndexes = nullptr;
}
