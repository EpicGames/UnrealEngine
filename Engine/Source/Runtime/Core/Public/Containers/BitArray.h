// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainerAllocationPolicies.h"
#include "CoreTypes.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryImageWriter.h"
#include "Serialization/MemoryLayout.h"
#include "Templates/EnableIf.h"
#include "Templates/Invoke.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"

class FPointerTableBase;
class FSHA1;
template<typename Allocator > class TBitArray;

// Functions for manipulating bit sets.
struct FBitSet
{
	/** Clears the next set bit in the mask and returns its index. */
	static UE_FORCEINLINE_HINT uint32 GetAndClearNextBit(uint32& Mask)
	{
		const uint32 LowestBitMask = (Mask) & (-(int32)Mask);
		const uint32 BitIndex = FMath::FloorLog2(LowestBitMask);
		Mask ^= LowestBitMask;
		return BitIndex;
	}

	// Clang generates 7 instructions for int32 DivideAndRoundUp but only 2 for uint32
	static constexpr uint32 BitsPerWord = NumBitsPerDWORD;

	UE_FORCEINLINE_HINT static uint32 CalculateNumWords(int32 NumBits)
	{
		checkSlow(NumBits >= 0);
		return FMath::DivideAndRoundUp(static_cast<uint32>(NumBits), BitsPerWord);
	}
};

// Forward declaration.
template<typename Allocator = FDefaultBitArrayAllocator,typename OtherAllocator = FDefaultBitArrayAllocator, bool Both=true>
class TConstDualSetBitIterator;
template<typename Allocator = FDefaultBitArrayAllocator>
class TBitArray;
template<typename Allocator = FDefaultBitArrayAllocator>
class TConstSetBitIterator;
template <typename AllocatorType, typename InDerivedType = void>
class TScriptBitArray;

/** Flag enumeration for control bitwise operator functionality */
enum class EBitwiseOperatorFlags
{
	/** Specifies that the result should be sized Max(A.Num(), B.Num()) */
	MaxSize = 1 << 0,
	/** Specifies that the result should be sized Min(A.Num(), B.Num()) */
	MinSize = 1 << 1,
	/** Only valid for self-mutating bitwise operators - indicates that the size of the LHS operand should not be changed */
	MaintainSize = 1 << 2,

	/** When MaxSize or MaintainSize is specified and the operands are sized differently, any missing bits in the resulting bit array will be considered as 1, rather than 0 */
	OneFillMissingBits = 1 << 4,
};
ENUM_CLASS_FLAGS(EBitwiseOperatorFlags)

namespace UE::Core::Private
{
	template <typename AllocatorA, typename AllocatorB, typename OutAllocator, typename ProjectionType>
	void BitwiseBinaryOperatorImpl(const TBitArray<AllocatorA>& InA, const TBitArray<AllocatorB>& InB, TBitArray<OutAllocator>& OutResult, EBitwiseOperatorFlags InFlags, ProjectionType&& InProjection)
	{
		check((const void*)&InA != (const void*)&InB && (const void*)&InA != (const void*)&OutResult && (const void*)&InB != (const void*)&OutResult);

		if (EnumHasAnyFlags(InFlags, EBitwiseOperatorFlags::MinSize))
		{

			const int32 MinNumBits = FMath::Min(InA.Num(), InB.Num());
			if (MinNumBits > 0)
			{
				OutResult.Reserve(MinNumBits);
				OutResult.NumBits = MinNumBits;

				typename TBitArray<AllocatorA>::FConstWordIterator IteratorA(InA);
				typename TBitArray<AllocatorB>::FConstWordIterator IteratorB(InB);

				typename TBitArray<OutAllocator>::FWordIterator IteratorResult(OutResult);

				for ( ; IteratorResult; ++IteratorResult, ++IteratorA, ++IteratorB)
				{
					const uint32 NewValue = Invoke(InProjection, IteratorA.GetWord(), IteratorB.GetWord());
					IteratorResult.SetWord(NewValue);
				}
			}

		}
		else if (EnumHasAnyFlags(InFlags, EBitwiseOperatorFlags::MaxSize))
		{

			const int32 MaxNumBits = FMath::Max(InA.Num(), InB.Num());
			const uint32 MissingBitsFill = EnumHasAnyFlags(InFlags, EBitwiseOperatorFlags::OneFillMissingBits) ? ~0u : 0;

			if (MaxNumBits)
			{
				OutResult.Reserve(MaxNumBits);
				OutResult.NumBits = MaxNumBits;

				typename TBitArray<AllocatorA>::FConstWordIterator IteratorA(InA);
				typename TBitArray<AllocatorB>::FConstWordIterator IteratorB(InB);

				IteratorA.FillMissingBits(MissingBitsFill);
				IteratorB.FillMissingBits(MissingBitsFill);

				typename TBitArray<OutAllocator>::FWordIterator IteratorResult(OutResult);

				for ( ; IteratorResult; ++IteratorResult, ++IteratorA, ++IteratorB)
				{
					uint32 A = IteratorA ? IteratorA.GetWord() : MissingBitsFill;
					uint32 B = IteratorB ? IteratorB.GetWord() : MissingBitsFill;

					IteratorResult.SetWord(Invoke(InProjection, A, B));
				}
			}

		}
		else
		{
			checkf(false, TEXT("Invalid size flag specified for binary bitwise AND"));
		}

		OutResult.CheckInvariants();
	}


	template<typename OtherAllocator, typename OutAllocator, typename ProjectionType>
	void BitwiseOperatorImpl(const TBitArray<OtherAllocator>& InOther, TBitArray<OutAllocator>& OutResult, EBitwiseOperatorFlags InFlags, ProjectionType&& InProjection)
	{
		if constexpr (std::is_same_v<OtherAllocator, OutAllocator>)
		{
			check(&InOther != &OutResult);
		}

		int32 NewNumBits = OutResult.NumBits;
		if (EnumHasAnyFlags(InFlags, EBitwiseOperatorFlags::MinSize))
		{
			NewNumBits = FMath::Min(InOther.Num(), OutResult.Num());
		}
		else if (EnumHasAnyFlags(InFlags, EBitwiseOperatorFlags::MaxSize))
		{
			NewNumBits = FMath::Max(InOther.Num(), OutResult.Num());
		}

		const int32 SizeDifference = NewNumBits - OutResult.NumBits;
		if (SizeDifference < 0)
		{
			OutResult.NumBits = NewNumBits;
			OutResult.ClearPartialSlackBits();
		}
		else if (SizeDifference > 0)
		{
			const bool bPadValue = EnumHasAnyFlags(InFlags, EBitwiseOperatorFlags::OneFillMissingBits);
			OutResult.Add(bPadValue, SizeDifference);
		}

		const uint32 MissingBitsFill = EnumHasAnyFlags(InFlags, EBitwiseOperatorFlags::OneFillMissingBits) ? ~0u : 0;
		if (OutResult.NumBits != 0)
		{
			typename TBitArray<OtherAllocator>::FConstWordIterator IteratorOther(InOther);
			IteratorOther.FillMissingBits(MissingBitsFill);

			typename TBitArray<OutAllocator>::FWordIterator IteratorResult(OutResult);

			for ( ; IteratorResult; ++IteratorResult, ++IteratorOther)
			{
				const uint32 OtherValue = IteratorOther ? IteratorOther.GetWord() : MissingBitsFill;
				IteratorResult.SetWord(Invoke(InProjection, IteratorResult.GetWord(), OtherValue));
			}
		}

		OutResult.CheckInvariants();
	}
}

/** Used to read/write a bit in the array as a bool. */
class FBitReference
{
public:

	UE_FORCEINLINE_HINT explicit FBitReference(uint32& InData UE_LIFETIMEBOUND,uint32 InMask)
	:	Data(InData)
	,	Mask(InMask)
	{}

	[[nodiscard]] UE_FORCEINLINE_HINT operator bool() const
	{
		 return (Data & Mask) != 0;
	}
	UE_FORCEINLINE_HINT void operator=(const bool NewValue)
	{
		if(NewValue)
		{
			Data |= Mask;
		}
		else
		{
			Data &= ~Mask;
		}
	}
	UE_FORCEINLINE_HINT void operator|=(const bool NewValue)
	{
		if (NewValue)
		{
			Data |= Mask;
		}
	}
	UE_FORCEINLINE_HINT void operator&=(const bool NewValue)
	{
		if (!NewValue)
		{
			Data &= ~Mask;
		}
	}
	UE_FORCEINLINE_HINT void AtomicSet(const bool NewValue)
	{
		uint32 Current = static_cast<uint32>(FPlatformAtomics::AtomicRead_Relaxed((const volatile int32*)&Data));
		if(NewValue)
		{
			if (!(Current & Mask))
			{
				FPlatformAtomics::InterlockedOr((volatile int32*)&Data, (int32)Mask);
			}
		}
		else
		{
			if (Current & Mask)
			{
				FPlatformAtomics::InterlockedAnd((volatile int32*)&Data, (int32)~Mask);
			}
		}
	}
	[[nodiscard]] UE_FORCEINLINE_HINT bool AtomicGet() const
	{
		const uint32 Current = static_cast<uint32>(FPlatformAtomics::AtomicRead_Relaxed((const volatile int32*)&Data));
		return (Current & Mask) != 0;
	}
	UE_FORCEINLINE_HINT FBitReference& operator=(const FBitReference& Copy)
	{
		// As this is emulating a reference, assignment should not rebind,
		// it should write to the referenced bit.
		*this = (bool)Copy;
		return *this;
	}

private:
	uint32& Data;
	uint32 Mask;
};


/** Used to read a bit in the array as a bool. */
class FConstBitReference
{
public:

	UE_FORCEINLINE_HINT explicit FConstBitReference(const uint32& InData UE_LIFETIMEBOUND,uint32 InMask)
	:	Data(InData)
	,	Mask(InMask)
	{}

	[[nodiscard]] UE_FORCEINLINE_HINT operator bool() const
	{
		 return (Data & Mask) != 0;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool AtomicGet() const
	{
		const uint32 Current = static_cast<uint32>(FPlatformAtomics::AtomicRead_Relaxed((const volatile int32*)&Data));
		return (Current & Mask) != 0;
	}

private:
	const uint32& Data;
	uint32 Mask;
};


/** Used to reference a bit in an unspecified bit array. */
class FRelativeBitReference
{
public:
	UE_FORCEINLINE_HINT explicit FRelativeBitReference(int32 BitIndex)
		: WordIndex(BitIndex >> NumBitsPerDWORDLogTwo)
		, Mask(1 << (BitIndex & (NumBitsPerDWORD - 1)))
	{
	}

	int32  WordIndex;
	uint32 Mask;

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(FRelativeBitReference Other) const
	{
		return (WordIndex == Other.WordIndex) & (Mask == Other.Mask);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(FRelativeBitReference Other) const
	{
		return !(Other == *this);
	}
};

class FBitArrayMemory
{
public:
	/**
	 * Copy NumBits bits from the source pointer and offset into the dest pointer and offset.
	 * This function is not suitable for general use because it uses a bit order that is specific to the uint32 internal storage of BitArray
	 *
	 * Bits within each word are read or written in the current platform's mathematical bitorder (Data[0] & 0x1, Data[0] & 0x2, ... Data[0] & 0x100, ... Data[0] & 0x80000000, Data[1] & 0x1 ...
	 * Correctly handles overlap between destination range and source range; the array of destination bits will be a copy of the source bits as they were before the copy started.
	 * @param DestBits The base location to which the bits are written.
	 * @param DestOffset The (word-order) bit within DestBits at which to start writing. Can be any value; offsets outside of [0,NumBitsPerDWORD) will be equivalent to modifying the DestBits pointer.
	 * @param SourceBits The base location from which the bits are read.
	 * @param SourceOffset The (word-order) bit within SourceBits at which to start reading. Can be any value; offsets outside of [0,NumBitsPerDWORD) will be equivalent to modifying the SourceBits pointer.
	 * @param NumBits Number of bits to copy. Must be >= 0.
	 */
	static CORE_API void MemmoveBitsWordOrder(uint32* DestBits, int32 DestOffset, const uint32* SourceBits, int32 SourceOffset, uint32 NumBits);
	static inline void MemmoveBitsWordOrder(int32* DestBits, int32 DestOffset, const int32* SourceBits, int32 SourceOffset, uint32 NumBits)
	{
		MemmoveBitsWordOrder(reinterpret_cast<uint32*>(DestBits), DestOffset, reinterpret_cast<const uint32*>(SourceBits), SourceOffset, NumBits);
	}

	/** Given Data and Offset that specify a specific bit in a specific word, modify Data and Offset so that they specify the same bit but that 0 <= Offset < NumBitsPerDWORD. */
	inline static void ModularizeWordOffset(uint32*& Data, int32& Offset)
	{
		ModularizeWordOffset(const_cast<uint32 const*&>(Data), Offset);
	}

	/** Given Data and Offset that specify a specific bit in a specific word, modify Data and Offset so that they specify the same bit but that 0 <= Offset < NumBitsPerDWORD. */
	static void ModularizeWordOffset(uint32 const*& Data, int32& Offset);

private:

	/**
	 * Copy NumBits bits from the source pointer at the given offset into the dest pointer at the given offset.
	 * It has the same behavior as MemmoveBitsWordOrder under the constaint that DestOffset == SourceOffset.
	 */
	static void MemmoveBitsWordOrderAlignedInternal(uint32*const StartDest, const uint32*const StartSource, int32 StartOffset, uint32 NumBits);

	friend class FBitArrayMemoryTest;
};

/**
 * A dynamically sized bit array.
 * An array of Booleans.  They stored in one bit/Boolean.  There are iterators that efficiently iterate over only set bits.
 */
template<typename Allocator /*= FDefaultBitArrayAllocator*/>
class TBitArray
{
	template <typename, typename>
	friend class TScriptBitArray;

	template <typename AllocatorA, typename AllocatorB, typename OutAllocator, typename ProjectionType>
	friend void UE::Core::Private::BitwiseBinaryOperatorImpl(const TBitArray<AllocatorA>&, const TBitArray<AllocatorB>&, TBitArray<OutAllocator>&, EBitwiseOperatorFlags, ProjectionType&&);

	template<typename OtherAllocator, typename OutAllocator, typename ProjectionType>
	friend void UE::Core::Private::BitwiseOperatorImpl(const TBitArray<OtherAllocator>&, TBitArray<OutAllocator>&, EBitwiseOperatorFlags, ProjectionType&&);

	typedef uint32 WordType;
	static constexpr WordType FullWordMask = (WordType)-1;

public:

	typedef typename Allocator::template ForElementType<uint32> AllocatorType;

	template<typename>
	friend class TConstSetBitIterator;

	template<typename,typename,bool>
	friend class TConstDualSetBitIterator;

	[[nodiscard]] constexpr TBitArray()
	:	NumBits(0)
	,	MaxBits(AllocatorInstance.GetInitialCapacity() * NumBitsPerDWORD)
	{
		// ClearPartialSlackBits is already satisfied since final word does not exist when NumBits == 0
	}

	[[nodiscard]] explicit consteval TBitArray(EConstEval)
	:	AllocatorInstance(ConstEval)
	, 	NumBits(0)
	,	MaxBits(AllocatorInstance.GetInitialCapacity() * NumBitsPerDWORD)
	{
	}

	/**
	 * Minimal initialization constructor.
	 * @param Value - The value to initial the bits to.
	 * @param InNumBits - The initial number of bits in the array.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT explicit TBitArray(bool bValue, int32 InNumBits)
	:	MaxBits(AllocatorInstance.GetInitialCapacity() * NumBitsPerDWORD)
	{
		Init(bValue, InNumBits);
	}

	/**
	 * Move constructor.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TBitArray(TBitArray&& Other)
	{
		this->Move(*this, Other);
	}

	/**
	 * Copy constructor.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TBitArray(const TBitArray& Copy)
	:	NumBits(0)
	,	MaxBits(0)
	{
		Assign<Allocator>(Copy);
	}

	template<typename OtherAllocator>
	[[nodiscard]] UE_FORCEINLINE_HINT explicit TBitArray(const TBitArray<OtherAllocator> & Copy)
		: NumBits(0)
		, MaxBits(0)
	{
		Assign<OtherAllocator>(Copy);
	}

	/**
	 * Move assignment.
	 */
	UE_FORCEINLINE_HINT TBitArray& operator=(TBitArray&& Other)
	{
		if (this != &Other)
		{
			this->Move(*this, Other);
		}

		return *this;
	}

	/**
	 * Assignment operator.
	 */
	UE_FORCEINLINE_HINT TBitArray& operator=(const TBitArray& Copy)
	{
		// check for self assignment since we don't use swap() mechanic
		if (this != &Copy)
		{
			Assign<Allocator>(Copy);
		}
		return *this;
	}


	template<typename OtherAllocator>
	UE_FORCEINLINE_HINT TBitArray& operator=(const TBitArray< OtherAllocator >& Copy)
	{
		// No need for a self-assignment check. If we were the same, we'd be using
		// the default assignment operator.
		Assign<OtherAllocator>(Copy);
		return *this;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TBitArray<Allocator>& Other) const
	{
		if (Num() != Other.Num())
		{
			return false;
		}

		return FMemory::Memcmp(GetData(), Other.GetData(), GetNumWords() * sizeof(uint32)) == 0;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator<(const TBitArray<Allocator>& Other) const
	{
		//sort by length
		if (Num() != Other.Num())
		{
			return Num() < Other.Num();
		}

		uint32 NumWords = GetNumWords();
		const uint32* Data0 = GetData();
		const uint32* Data1 = Other.GetData();

		//lexicographically compare
		for (uint32 i = 0; i < NumWords; i++)
		{
			if (Data0[i] != Data1[i])
			{
				return Data0[i] < Data1[i];
			}
		}
		return false;
	}

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TBitArray<Allocator>& Other) const
	{
		return !(*this == Other);
	}
#endif

private:
	[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetNumWords() const
	{
		return FBitSet::CalculateNumWords(NumBits);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetMaxWords() const
	{
		return FBitSet::CalculateNumWords(MaxBits);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetLastWordMask(int32 EndIndexExclusive) const
	{
		const uint32 UnusedBits = (FBitSet::BitsPerWord - static_cast<uint32>(EndIndexExclusive) % FBitSet::BitsPerWord) % FBitSet::BitsPerWord;
		return ~0u >> UnusedBits;
	}

	UE_FORCEINLINE_HINT static void SetWords(uint32* Words, int32 NumWords, bool bValue)
	{
		if (NumWords > 8)
		{
			FMemory::Memset(Words, bValue ? 0xff : 0, NumWords * sizeof(uint32));
		}
		else
		{
			uint32 Word = bValue ? ~0u : 0u;
			for (int32 Idx = 0; Idx < NumWords; ++Idx)
			{
				Words[Idx] = Word;
			}
		}
	}

	template <typename BitArrayType>
	static UE_FORCEINLINE_HINT void Move(BitArrayType& ToArray, BitArrayType& FromArray)
	{
		ToArray.AllocatorInstance.MoveToEmpty(FromArray.AllocatorInstance);

		ToArray  .NumBits = FromArray.NumBits;
		ToArray  .MaxBits = FromArray.MaxBits;
		FromArray.NumBits = 0;
		FromArray.MaxBits = 0;
		// No need to call this.ClearPartialSlackBits, because the words we're copying or moving from satisfy the invariant
		// No need to call FromArray.ClearPartialSlackBits because NumBits == 0 automatically satisfies the invariant
	}

	template<typename OtherAllocator>
	void Assign(const TBitArray<OtherAllocator>& Other)
	{
		Empty(Other.Num());
		NumBits = Other.Num();
		if (NumBits)
		{
			FMemory::Memcpy(GetData(), Other.GetData(), GetNumWords() * sizeof(uint32));
			// No need to call ClearPartialSlackBits, because the bits we're copying from satisfy the invariant
			// If NumBits == 0, the invariant is automatically satisfied so we also don't need ClearPartialSlackBits in that case.
		}
	}


public:

	/**
	 * Checks the invariants of this class
	 */
	void CheckInvariants() const
	{
#if DO_CHECK
		checkf(NumBits <= MaxBits, TEXT("TBitArray::NumBits (%d) should never be greater than MaxBits (%d)"), NumBits, MaxBits);
		checkf(NumBits >= 0 && MaxBits >= 0, TEXT("NumBits (%d) and MaxBits (%d) should always be >= 0"), NumBits, MaxBits);

		// Verify the ClearPartialSlackBits invariant
		const int32 UsedBits = (NumBits % NumBitsPerDWORD);
		if (UsedBits != 0)
		{
			const int32 LastWordIndex = NumBits / NumBitsPerDWORD;
			const uint32 SlackMask = FullWordMask << UsedBits;

			const uint32 LastWord = *(GetData() + LastWordIndex);
			checkf((LastWord & SlackMask) == 0, TEXT("TBitArray slack bits are non-zero, this will result in undefined behavior."));
		}
#endif
	}

	/**
	 * Serializer
	 */
	void Serialize(FArchive& Ar)
	{
		// serialize number of bits
		Ar << NumBits;

		if (Ar.IsLoading())
		{
			// no need for slop when reading; set MaxBits to the smallest legal value that is >= NumBits
			MaxBits = NumBitsPerDWORD * FMath::Max(FBitSet::CalculateNumWords(NumBits), (uint32)AllocatorInstance.GetInitialCapacity());

			// allocate room for new bits
			Realloc(0);
		}

		// serialize the data as one big chunk
		Ar.Serialize(GetData(), GetNumWords() * sizeof(uint32));

		if (Ar.IsLoading() && !Ar.IsObjectReferenceCollector() && !Ar.IsCountingMemory())
		{
			// Clear slack bits incase they were serialized non-null
			ClearPartialSlackBits();
		}
	}

	/**
	 * Adds a bit to the array with the given value.
	 * @return The index of the added bit.
	 */
	int32 Add(const bool Value)
	{
		const int32 Index = AddUninitialized(1);
		SetBitNoCheck(Index, Value);
		return Index;
	}

	/**
	 * Adds multiple bits to the array with the given value.
	 * @return The index of the first added bit.
	 */
	int32 Add(const bool Value, int32 NumBitsToAdd)
	{
		// Support for legacy behavior requires us to silently ignore NumBitsToAdd < 0
		if (NumBitsToAdd < 0)
		{
			return NumBits;
		}
		const int32 Index = AddUninitialized(NumBitsToAdd);
		SetRange(Index, NumBitsToAdd, Value);
		return Index;
	}

	/**
	 * Adds multiple bits read from the given pointer.
	 * @param ReadBits The address of sized integers to read the bits from.
	 *                 Bits are read from ReadBits in the current platform's mathematical bitorder (ReadBits[0] & 0x1, ReadBits[0] & 0x2, ... ReadBits[0] & 0x100, ... ReadBits[0] & 0x80000000, ReadBits[1] & 0x1 ...
	 * @param NumBitsToAdd The number of bits to add. Must be >= 0.
	 * @param ReadOffsetBits Number of bits into ReadBits at which to start reading. Must be >= 0.
	 * @return The index of the first added bit.
	 */
	template <typename InWordType>
	int32 AddRange(const InWordType* ReadBits, int32 NumBitsToAdd, int32 ReadOffsetBits = 0)
	{
		const int32 Index = AddUninitialized(NumBitsToAdd);
		SetRangeFromRange(Index, NumBitsToAdd, ReadBits, ReadOffsetBits);
		return Index;
	}

	/**
	 * Adds multiple bits read from the given BitArray.
	 * @param ReadBits The value to set the bits to.
	 * @param NumBitsToAdd The number of bits to add. Must be >= 0.
	 * @param ReadOffsetBits Number of bits into ReadBits at which to start reading. Must be >= 0.
	 * @return The index of the first added bit.
	 */
	template <typename OtherAllocator>
	int32 AddRange(const TBitArray<OtherAllocator>& ReadBits, int32 NumBitsToAdd, int32 ReadOffsetBits = 0)
	{
		check(0 <= ReadOffsetBits && ReadOffsetBits + NumBitsToAdd <= ReadBits.NumBits);
		const int32 Index = AddUninitialized(NumBitsToAdd);
		SetRangeFromRange(Index, NumBitsToAdd, ReadBits, ReadOffsetBits);
		return Index;
	}

	/**
	 * Inserts space for multiple bits at the end of the array.
	 * The inserted bits are set to arbitrary values and should be written using SetRange or otherwise before being read.
	 * @param NumBitsToAdd The number of bits to add. Must be >= 0.
	 */
	int32 AddUninitialized(int32 NumBitsToAdd)
	{
		check(NumBitsToAdd >= 0);
		int32 AddedIndex = NumBits;
		if (NumBitsToAdd > 0)
		{
			int32 OldLastWordIndex = NumBits == 0 ? -1 : (NumBits - 1) / NumBitsPerDWORD;
			int32 NewLastWordIndex = (NumBits + NumBitsToAdd - 1) / NumBitsPerDWORD;
			if (NewLastWordIndex == OldLastWordIndex)
			{
				// We're not extending into a new word, so we don't need to reserve more memory and we don't need to clear the unused bits on the final word
				NumBits += NumBitsToAdd;
			}
			else
			{
				Reserve(NumBits + NumBitsToAdd);
				NumBits += NumBitsToAdd;
				ClearPartialSlackBits();
			}
		}
		return AddedIndex;
	}

	/**
	 * Adds a bit with the given value at the given index in the array.
	 * @param Value The value of the bit to add
	 * @param Index - The index at which to add; must be 0 <= Index <= Num().
	 */
	void Insert(bool Value, int32 Index)
	{
		InsertUninitialized(Index, 1);
		SetBitNoCheck(Index, Value);
	}

	/**
	 * Inserts multiple bits with the given value into the array, starting at the given Index.
	 * @param Value The value of the bits to add
	 * @param Index The index at which to add; must be 0 <= Index <= Num().
	 * @param NumBitsToAdd The number of bits to add. Must be >= 0.
	 */
	void Insert(bool Value, int32 Index, int32 NumBitsToAdd)
	{
		InsertUninitialized(Index, NumBitsToAdd);
		SetRange(Index, NumBitsToAdd, Value);
	}

	/**
	 * Inserts multiple bits read from the given pointer, starting at the given index.
	 * @param ReadBits The address of sized integers to read the bits from.
	 *                 Bits are read from ReadBits in the current platform's mathematical bitorder (ReadBits[0] & 0x1, ReadBits[0] & 0x2, ... ReadBits[0] & 0x100, ... ReadBits[0] & 0x80000000, ReadBits[1] & 0x1 ...
	 * @param Index The index at which to add; must be 0 <= Index <= Num().
	 * @param NumBitsToAdd The number of bits to add. Must be >= 0.
	 * @param ReadOffsetBits Number of bits into ReadBits at which to start reading.
	 */
	template <typename InWordType>
	void InsertRange(const InWordType* ReadBits, int32 Index, int32 NumBitsToAdd, int32 ReadOffsetBits = 0)
	{
		InsertUninitialized(Index, NumBitsToAdd);
		SetRangeFromRange(Index, NumBitsToAdd, ReadBits, ReadOffsetBits);
	}

	/**
	 * Inserts multiple bits read from the given BitArray into the array, starting at the given index.
	 * @param ReadBits The value to set the bits to.
	 * @param Index The index at which to add; must be 0 <= Index <= Num().
	 * @param NumBitsToAdd The number of bits to add. Must be >= 0.
	 * @param ReadOffsetBits Number of bits into ReadBits at which to start reading.
	 */
	template <typename OtherAllocator>
	void InsertRange(const TBitArray<OtherAllocator>& ReadBits, int32 Index, int32 NumBitsToAdd, int32 ReadOffsetBits = 0)
	{
		check(0 <= ReadOffsetBits && ReadOffsetBits + NumBitsToAdd <= ReadBits.NumBits);
		InsertUninitialized(Index, NumBitsToAdd);
		SetRangeFromRange(Index, NumBitsToAdd, ReadBits, ReadOffsetBits);
	}

	/**
	 * Inserts space for multiple bits into the array, starting at the given index.
	 * The inserted bits are set to arbitrary values and should be written using SetRange or otherwise before being read.
	 * @param Index The index at which to add; must be 0 <= Index <= Num().
	 * @param NumBitsToAdd The number of bits to add. Must be >= 0.
	 */
	void InsertUninitialized(int32 Index, int32 NumBitsToAdd)
	{
		check(0 <= Index && Index <= NumBits);
		check(NumBitsToAdd >= 0);

		if (NumBitsToAdd > 0)
		{
			uint32 OldNumBits = NumBits;
			AddUninitialized(NumBitsToAdd);
			uint32 NumToShift = OldNumBits - Index;
			if (NumToShift > 0)
			{
				// MemmoveBitsWordOrder handles overlapping source and dest
				FBitArrayMemory::MemmoveBitsWordOrder(GetData(), Index + NumBitsToAdd, GetData(), Index, NumToShift);
			}
		}
	}

	/**
	 * Removes all bits from the array, potentially leaving space allocated for an expected number of bits about to be added.
	 * @param ExpectedNumBits - The expected number of bits about to be added.
	 */
	void Empty(int32 ExpectedNumBits = 0)
	{
		ExpectedNumBits = static_cast<int32>(FBitSet::CalculateNumWords(ExpectedNumBits)) * NumBitsPerDWORD;
		const int32 InitialMaxBits = AllocatorInstance.GetInitialCapacity() * NumBitsPerDWORD;

		NumBits = 0;

		// If we need more bits or can shrink our allocation, do so
		// Otherwise, reuse current initial allocation
		if (ExpectedNumBits > MaxBits || MaxBits > InitialMaxBits)
		{
			MaxBits = FMath::Max(ExpectedNumBits, InitialMaxBits);
			Realloc(0);
		}
	}

	/**
	 * Reserves memory such that the array can contain at least Number bits.
	 *
	 * @Number  The number of bits to reserve space for.
	 */
	void Reserve(int32 Number)
	{
		if (Number > MaxBits)
		{
			const uint32 MaxWords = AllocatorInstance.CalculateSlackGrow(
				FBitSet::CalculateNumWords(Number),
				GetMaxWords(),
				sizeof(uint32)
				);
			MaxBits = MaxWords * NumBitsPerDWORD;
			Realloc(NumBits);
		}
	}

	/**
	 * Removes all bits from the array retaining any space already allocated.
	 */
	void Reset()
	{
		NumBits = 0;
	}

	/**
	 * Resets the array's contents. Use TBitArray(bool bValue, int32 InNumBits) instead of default constructor and Init().
	 *
	 * @param Value - The value to initial the bits to.
	 * @param NumBits - The number of bits in the array.
	 */
	UE_FORCEINLINE_HINT void Init(bool bValue, int32 InNumBits)
	{
		NumBits = InNumBits;
		
		const uint32 NumWords = GetNumWords();
		const uint32 MaxWords = GetMaxWords();

		if (NumWords > 0)
		{
			if (NumWords > MaxWords)
			{
				AllocatorInstance.ResizeAllocation(0, NumWords, sizeof(uint32));
				MaxBits = NumWords * NumBitsPerDWORD;
			}

			SetWords(GetData(), NumWords, bValue);
			ClearPartialSlackBits();
		}
	}

	/** Sets number of bits without initializing new bits. */
	void SetNumUninitialized(int32 InNumBits)
	{
		int32 PreviousNumBits = NumBits;
		NumBits = InNumBits;

		if (InNumBits > MaxBits)
		{
			const int32 PreviousNumWords = FBitSet::CalculateNumWords(PreviousNumBits);
			const uint32 MaxWords = AllocatorInstance.CalculateSlackReserve(
				FBitSet::CalculateNumWords(InNumBits), sizeof(uint32));
			
			AllocatorInstance.ResizeAllocation(PreviousNumWords, MaxWords, sizeof(uint32));	

			MaxBits = MaxWords * NumBitsPerDWORD;
		}

		ClearPartialSlackBits();
	}

	/** Sets the number of bits, initializing any potentially added bits to the given value. */
	template <typename ValueType>
	FORCENOINLINE void SetNum(int32 InNumBits, ValueType bValue)
	{
		static_assert(std::is_same_v<ValueType, bool>, "TBitArray::SetNum: unexpected type passed as the bValue argument (expected bool)");
		int32 PreviousNumBits = NumBits;
		SetNumUninitialized(InNumBits);
		if (InNumBits > PreviousNumBits)
		{
			SetRange(PreviousNumBits, InNumBits - PreviousNumBits, bValue);
		}
	}

	/**
	 * Sets or unsets a range of bits within the array.
	 * @param Index  The index of the first bit to set; must be 0 <= Index <= Num().
	 * @param NumBitsToSet The number of bits to set, must satisify Index + NumBitsToSet <= Num().
	 * @param  Value  The value to set the bits to.
	 */
	FORCENOINLINE void SetRange(int32 Index, int32 NumBitsToSet, bool Value)
	{
		check(Index >= 0 && NumBitsToSet >= 0 && Index + NumBitsToSet <= NumBits);

		if (NumBitsToSet == 0)
		{
			return;
		}

		// Work out which uint32 index to set from, and how many
		uint32 StartIndex = Index / NumBitsPerDWORD;
		uint32 Count      = (Index + NumBitsToSet + (NumBitsPerDWORD - 1)) / NumBitsPerDWORD - StartIndex;

		// Work out masks for the start/end of the sequence
		uint32 StartMask  = FullWordMask << (Index % NumBitsPerDWORD);
		uint32 EndMask    = FullWordMask >> (NumBitsPerDWORD - (Index + NumBitsToSet) % NumBitsPerDWORD) % NumBitsPerDWORD;

		uint32* Data = GetData() + StartIndex;
		if (Value)
		{
			if (Count == 1)
			{
				*Data |= StartMask & EndMask;
			}
			else
			{
				*Data++ |= StartMask;
				Count -= 2;
				while (Count != 0)
				{
					*Data++ = ~0u;
					--Count;
				}
				*Data |= EndMask;
			}
		}
		else
		{
			if (Count == 1)
			{
				*Data &= ~(StartMask & EndMask);
			}
			else
			{
				*Data++ &= ~StartMask;
				Count -= 2;
				while (Count != 0)
				{
					*Data++ = 0;
					--Count;
				}
				*Data &= ~EndMask;
			}
		}

		CheckInvariants();
	}

	/**
	 * Sets range of bits within the TBitArray to the values read out of a pointer.
	 * @param Index  The index of the first bit to set; must be 0 <= Index <= Num().
	 * @param NumBitsToSet The number of bits to set, must satisify 0 <= NumBitsToSet && Index + NumBitsToSet <= Num().
	 * @param ReadBits The address of sized integers to read the bits from.
	 *                 Bits are read from ReadBits in the current platform's mathematical bitorder (ReadBits[0] & 0x1, ReadBits[0] & 0x2, ... ReadBits[0] & 0x100, ... ReadBits[0] & 0x80000000, ReadBits[1] & 0x1 ...
	 * @param ReadOffsetBits Number of bits into ReadBits at which to start reading. 
	 */
	template <typename InWordType>
	void SetRangeFromRange(int32 Index, int32 NumBitsToSet, const InWordType* ReadBits, int32 ReadOffsetBits = 0)
	{
		check(Index >= 0 && NumBitsToSet >= 0 && Index + NumBitsToSet <= NumBits);
		check(NumBitsToSet == 0 || ReadBits != nullptr);

		// The planned implementation for big endian:
		// iterate over each InWordType in ReadBits, and upcast it to the TBitArray's WordType. Call MemmoveBitsWordOrder(GetData(), Index + n, &UpcastedWord, ReadOffsetBits, Min(NumBitsToSet, BitsPerWord-ReadOffsetBits) for each word
		static_assert(PLATFORM_LITTLE_ENDIAN, "SetRangeFromRange does not yet support big endian platforms");
		FBitArrayMemory::MemmoveBitsWordOrder(GetData(), Index, ReadBits, ReadOffsetBits, NumBitsToSet);
	}

	/**
	 * Sets range of bits within this TBitArray to the values read out another TBitArray.
	 * @param Index  The index of the first bit to set; must be 0 <= Index <= Num().
	 * @param NumBitsToSet The number of bits to set, must satisify 0 <= NumBitsToSet && Index + NumBitsToSet <= Num().
	 * @param ReadBits The value to set the bits to.
	 * @param ReadOffsetBits Number of bits into ReadBits at which to start reading.
	 */
	template <typename OtherAllocator>
	UE_FORCEINLINE_HINT void SetRangeFromRange(int32 Index, int32 NumBitsToSet, const TBitArray<OtherAllocator>& ReadBits, int32 ReadOffsetBits = 0)
	{
		check(Index >= 0 && NumBitsToSet >= 0 && Index + NumBitsToSet <= NumBits);
		check(0 <= ReadOffsetBits && ReadOffsetBits + NumBitsToSet <= ReadBits.NumBits);
		FBitArrayMemory::MemmoveBitsWordOrder(GetData(), Index, ReadBits.GetData(), ReadOffsetBits, NumBitsToSet);
	}

	/**
	 * Reads a range of bits within the array and writes them to the given pointer.
	 * @param Index  The index of the first bit to read; must be 0 <= Index <= Num().
	 * @param NumBitsToGet The number of bits to read, must satisify 0 <= NumBitsToGet && Index + NumBitsToGet <= Num().
	 * @param WriteBits The address of sized integers to write the bits to.
	 *                  Bits are written into WriteBits in the current platform's mathematical bitorder (WriteBits[0] & 0x1, WriteBits[0] & 0x2, ... WriteBits[0] & 0x100, ... WriteBits[0] & 0x80000000, WriteBits[1] & 0x1 ...
	 * @param WriteOffsetBits Number of bits into WriteBits at which to start writing.
	 */
	template <typename InWordType>
	UE_FORCEINLINE_HINT void GetRange(int32 Index, int32 NumBitsToGet, InWordType* WriteBits, int32 WriteOffsetBits = 0) const
	{
		check(Index >= 0 && NumBitsToGet >= 0 && Index + NumBitsToGet <= NumBits);
		check(NumBitsToGet == 0 || WriteBits != nullptr);

		// See SetRangeFromRange for notes on the planned big endian implementation
		static_assert(PLATFORM_LITTLE_ENDIAN, "SetRangeFromRange does not yet support big endian platforms");
		FBitArrayMemory::MemmoveBitsWordOrder(WriteBits, WriteOffsetBits, GetData(), Index, NumBitsToGet);
	}

	/**
	 * Removes bits from the array.
	 * @param BaseIndex - The index of the first bit to remove.
	 * @param NumBitsToRemove - The number of consecutive bits to remove.
	 */
	void RemoveAt(int32 BaseIndex,int32 NumBitsToRemove = 1)
	{
		check(BaseIndex >= 0 && NumBitsToRemove >= 0 && BaseIndex + NumBitsToRemove <= NumBits);

		if (BaseIndex + NumBitsToRemove != NumBits)
		{
			// MemmoveBitsWordOrder handles overlapping source and dest
			uint32 NumToShift = NumBits - (BaseIndex + NumBitsToRemove);
			FBitArrayMemory::MemmoveBitsWordOrder(GetData(), BaseIndex, GetData(), BaseIndex + NumBitsToRemove, NumToShift);
		}

		NumBits -= NumBitsToRemove;

		ClearPartialSlackBits();
		CheckInvariants();
	}

	/* Removes bits from the array by swapping them with bits at the end of the array.
	 * This is mainly implemented so that other code using TArray::RemoveSwap will have
	 * matching indices.
 	 * @param BaseIndex - The index of the first bit to remove.
	 * @param NumBitsToRemove - The number of consecutive bits to remove.
	 */
	void RemoveAtSwap( int32 BaseIndex, int32 NumBitsToRemove=1 )
	{
		check(BaseIndex >= 0 && NumBitsToRemove >= 0 && BaseIndex + NumBitsToRemove <= NumBits);
		if( BaseIndex < NumBits - NumBitsToRemove )
		{
			// Copy bits from the end to the region we are removing
			for( int32 Index=0;Index<NumBitsToRemove;Index++ )
			{
#if PLATFORM_MAC || PLATFORM_LINUX
				// Clang compiler doesn't understand the short syntax, so let's be explicit
				int32 FromIndex = NumBits - NumBitsToRemove + Index;
				FConstBitReference From(GetData()[FromIndex / NumBitsPerDWORD],1 << (FromIndex & (NumBitsPerDWORD - 1)));

				int32 ToIndex = BaseIndex + Index;
				FBitReference To(GetData()[ToIndex / NumBitsPerDWORD],1 << (ToIndex & (NumBitsPerDWORD - 1)));

				To = (bool)From;
#else
				(*this)[BaseIndex + Index] = (bool)(*this)[NumBits - NumBitsToRemove + Index];
#endif
			}
		}

		// Remove the bits from the end of the array.
		NumBits -= NumBitsToRemove;

		ClearPartialSlackBits();
		CheckInvariants();
	}
	

	/** 
	 * Helper function to return the amount of memory allocated by this container 
	 * @return number of bytes allocated by this container
	 */
	[[nodiscard]] uint32 GetAllocatedSize( void ) const
	{
		return FBitSet::CalculateNumWords(MaxBits) * sizeof(uint32);
	}

	/** Tracks the container's memory use through an archive. */
	void CountBytes(FArchive& Ar) const
	{
		Ar.CountBytes(
			GetNumWords() * sizeof(uint32),
			GetMaxWords() * sizeof(uint32)
		);
	}

	/**
	 * Finds the first occurrence of the specified value (true/false) in the array, and returns the bit index.
	 * If the specified value is not found, INDEX_NONE is returned.
	 *
	 * @param  bValue  The value (true/false) to search for.
	 *
	 * @return The index of the first occurrence of the specified value (true/false), or INDEX_NONE if not found.
	 */
	[[nodiscard]] int32 Find(bool bValue) const
	{
		return FindFromImpl(bValue, 0, NumBits);
	}

private:

	int32 FindFromImpl(bool bValue, int32 StartIndex, int32 EndIndexExclusive) const
	{
		// Produce a mask for the first iteration
		uint32 Mask = ~0u << (StartIndex % FBitSet::BitsPerWord);

		// Iterate over the array until we see a word with a matching bit
		const uint32 Test = bValue ? 0u : ~0u;

		const uint32* RESTRICT DwordArray = GetData();
		const int32 DwordCount = FBitSet::CalculateNumWords(EndIndexExclusive);
		int32 DwordIndex = FMath::DivideAndRoundDown(StartIndex, NumBitsPerDWORD);
		while (DwordIndex < DwordCount && (DwordArray[DwordIndex] & Mask) == (Test & Mask))
		{
			++DwordIndex;
			Mask = ~0u;
		}

		if (DwordIndex < DwordCount)
		{
			// If we're looking for a false, then we flip the bits - then we only need to find the first one bit
			const uint32 Bits = (bValue ? DwordArray[DwordIndex] : ~DwordArray[DwordIndex]) & Mask;
			UE_ASSUME(Bits != 0);
			const int32 LowestBitIndex = FMath::CountTrailingZeros(Bits) + (DwordIndex << NumBitsPerDWORDLogTwo);
			if (LowestBitIndex < EndIndexExclusive)
			{
				return LowestBitIndex;
			}
		}

		return INDEX_NONE;
	}

	int32 FindLastFromImpl(bool bValue, int32 EndIndexExclusive) const
	{
		// We need to early out on empty bit arrays in order to avoid iterating on the first (inexistent) DWord :
		if (NumBits == 0)
		{
			return INDEX_NONE;
		}

		// Produce a mask for the first iteration
		uint32 Mask = GetLastWordMask(EndIndexExclusive);

		// Iterate over the array until we see a word with a matching bit
		const uint32* RESTRICT DwordArray = GetData();
		uint32 DwordIndex = FBitSet::CalculateNumWords(EndIndexExclusive);
		const uint32 Test = bValue ? 0u : ~0u;
		for (;;)
		{
			if (DwordIndex == 0)
			{
				return INDEX_NONE;
			}
			--DwordIndex;
			if ((DwordArray[DwordIndex] & Mask) != (Test & Mask))
			{
				break;
			}
			Mask = ~0u;
		}

		// If we're looking for a false, then we flip the bits - then we only need to find the last one bit
		const uint32 Bits = (bValue ? DwordArray[DwordIndex] : ~DwordArray[DwordIndex]) & Mask;
		UE_ASSUME(Bits != 0);

		uint32 BitIndex = (NumBitsPerDWORD - 1) - FMath::CountLeadingZeros(Bits);

		int32 Result = BitIndex + (DwordIndex << NumBitsPerDWORDLogTwo);
		return Result;
	}

public:

	/**
	 * Finds the first occurrence of the specified value (true/false) in the array, starting from the given bit index, and returns the bit index.
	 * If the specified value is not found from the given index, INDEX_NONE is returned.
	 *
	 * @param  bValue      The value (true/false) to search for.
	 * @param  StartIndex  The index to start the search from.
	 *
	 * @pre StartIndex is expected to be in the inclusive range [0, Num()].
	 *
	 * @return The index of the first occurrence of the specified value (true/false) from StartIndex, or INDEX_NONE if not found.
	 */
	template <typename IndexType>
	[[nodiscard]] UE_FORCEINLINE_HINT int32 FindFrom(bool bValue, IndexType StartIndex) const
	{
		static_assert(!std::is_same_v<IndexType, bool>, "TBitArray::FindFrom: unexpected bool passed as the StartIndex argument");
		checkSlow((StartIndex >= 0) && (StartIndex <= NumBits));
		return FindFromImpl(bValue, StartIndex, NumBits);
	}

	/**
	* Finds the first occurrence of the specified value (true/false) in the array, starting from the given bit index, and returns the bit index.
	* If the specified value is not found from the given index, INDEX_NONE is returned.
	*
	* @param  bValue      The value (true/false) to search for.
	* @param  StartIndex  The index to start the search from.
	* @param  EndIndexExclusive	  The index to end the search at.
	*
	* @pre StartIndex is expected to be in the inclusive range [0, Num()].
	* @pre EndIndexExclusive is expected to be in the inclusive range [0, Num()].
	*
	* @return The index of the first occurrence of the specified value (true/false) from StartIndex to EndIndexExclusive, or INDEX_NONE if not found.
	*/
	template <typename IndexType>
	[[nodiscard]] UE_FORCEINLINE_HINT int32 FindFrom(bool bValue, IndexType StartIndex, IndexType EndIndexExclusive) const
	{
		static_assert(!std::is_same_v<IndexType, bool>, "TBitArray::FindFrom: unexpected bool passed as the StartIndex argument");
		checkSlow((StartIndex >= 0) && (StartIndex <= NumBits));
		checkSlow((EndIndexExclusive >= 0) && (EndIndexExclusive <= NumBits));
		return FindFromImpl(bValue, StartIndex, EndIndexExclusive);
	}

	/**
	 * Finds the last occurrence of the specified value (true/false) in the array, and returns the bit index.
	 * If the specified value is not found, INDEX_NONE is returned.
	 *
	 * @param  bValue  The value (true/false) to search for.
	 *
	 * @return The index of the last occurrence of the specified value (true/false), or INDEX_NONE if not found.
	 */
	[[nodiscard]] int32 FindLast(bool bValue) const 
	{
		return FindLastFromImpl(bValue, /*EndIndexExclusive = */NumBits);
	}

	/**
	 * Finds the last occurrence of the specified value (true/false) in the array starting the (reverse) search from the given bit index, and returns the bit index.
	 * If the specified value is not found, INDEX_NONE is returned.
	 *
	 * @param  bValue  The value (true/false) to search for.
	 * @param  EndIndexInclusive  The index to start the (reverse) search from.
	 *
	 * @return The index of the last occurrence of the specified value (true/false) from EndIndexInclusive, or INDEX_NONE if not found.
	 */
	template <typename IndexType>
	[[nodiscard]] int32 FindLastFrom(bool bValue, IndexType EndIndexInclusive) const
	{
		static_assert(!std::is_same_v<IndexType, bool>, "TBitArray::FindLastFrom: unexpected bool passed as the EndIndexInclusive argument");
		// This range check here ([-1, NumBits)) is because the end index is inclusive and because this is the reverse logic of the FindFrom range ([0, NumBits]) 
		//  It doesn't make much sense to allow EndIndexInclusive to be -1 in FindLastFrom (since that will always fail), much like it doesn't make sense to allow
		//  StartIndex to be NumBits in FindFrom (for the same reason) but this keeps the implementations consistent
		checkSlow((EndIndexInclusive >= -1) && (EndIndexInclusive < NumBits)); 
		return FindLastFromImpl(bValue, /*EndIndexExclusive = */EndIndexInclusive + 1);
	}

	/**
	 * Checks if the array contains the specified value (true/false).
	 *
	 * @param  bValue  The value (true/false) to check for.
	 *
	 * @return true if the array contains the specified value, false otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool Contains(bool bValue) const
	{
		return Find(bValue) != INDEX_NONE;
	}

	/**
	 * Finds the first occurrence of a zero bit in the array and sets it to one, returning the bit index.
	 * If there are no zero bits in the array, INDEX_NONE is returned.
	 *
	 * @param  StartIndex  The index to start the search from. Defaults to 0.
	 *
	 * @return The index of the first occurrence of a zero bit that was successfully set to one, or INDEX_NONE if not found.
	 */
	int32 FindAndSetFirstZeroBit(int32 StartIndex = 0)
	{
		const int32 FirstZeroBitIndex = FindFromImpl(false, StartIndex, NumBits);
		if (FirstZeroBitIndex != INDEX_NONE)
		{
			(*this)[FirstZeroBitIndex] = true;
			CheckInvariants();
		}
		return FirstZeroBitIndex;
	}

	/**
	 * Finds the last occurrence of a zero bit in the array and sets it to one, returning the bit index.
	 * If there are no zero bits in the array, INDEX_NONE is returned.
	 *
	 * @return The index of the last occurrence of a zero bit that was successfully set to one, or INDEX_NONE if not found.
	 */
	int32 FindAndSetLastZeroBit()
	{
		const int32 LastZeroBitIndex = FindLast(false);
		if (LastZeroBitIndex != INDEX_NONE)
		{
			(*this)[LastZeroBitIndex] = true;
			CheckInvariants();
		}
		return LastZeroBitIndex;
	}

	/**
	 * Return the bitwise AND of two bit arrays. The resulting bit array will be sized according to InFlags.
	 */
	template <typename AllocatorA, typename AllocatorB>
	[[nodiscard]] static TBitArray BitwiseAND(const TBitArray<AllocatorA>& A, const TBitArray<AllocatorB>& B, EBitwiseOperatorFlags InFlags)
	{
		TBitArray Result;
		UE::Core::Private::BitwiseBinaryOperatorImpl(A, B, Result, InFlags, [](uint32 InA, uint32 InB) { return InA & InB; });
		return Result;
	}

	/**
	 * Perform a custom bitwise operation on this bit array with another. This array receives the result and will be sized max(A.Num(), B.Num()).
	 */
	template <typename OtherAllocator, typename BinaryOpType>
	TBitArray& CombineWithBinaryOp(const TBitArray<OtherAllocator>& InOther, EBitwiseOperatorFlags InFlags, BinaryOpType&& InBinaryOp)
	{
		UE::Core::Private::BitwiseOperatorImpl(InOther, *this, InFlags, Forward<BinaryOpType>(InBinaryOp));
		return *this;
	}

	/**
	 * Perform a bitwise AND on this bit array with another. This array receives the result and will be sized max(A.Num(), B.Num()).
	 */
	template <typename OtherAllocator>
	TBitArray& CombineWithBitwiseAND(const TBitArray<OtherAllocator>& InOther, EBitwiseOperatorFlags InFlags)
	{
		UE::Core::Private::BitwiseOperatorImpl(InOther, *this, InFlags, [](uint32 InA, uint32 InB) { return InA & InB; });
		return *this;
	}

	/**
	 * Return the bitwise OR of two bit arrays. The resulting bit array will be sized according to InFlags.
	 */
	template <typename AllocatorA, typename AllocatorB>
	[[nodiscard]] static TBitArray BitwiseOR(const TBitArray<AllocatorA>& A, const TBitArray<AllocatorB>& B, EBitwiseOperatorFlags InFlags)
	{
		if constexpr (std::is_same_v<AllocatorA, AllocatorB>)
		{
			check(&A != &B);
		}

		TBitArray Result;
		UE::Core::Private::BitwiseBinaryOperatorImpl(A, B, Result, InFlags, [](uint32 InA, uint32 InB) { return InA | InB; });
		return Result;
	}

	/**
	 * Return the bitwise OR of two bit arrays. The resulting bit array will be sized according to InFlags.
	 */
	template <typename OtherAllocator>
	TBitArray& CombineWithBitwiseOR(const TBitArray<OtherAllocator>& InOther, EBitwiseOperatorFlags InFlags)
	{
		UE::Core::Private::BitwiseOperatorImpl(InOther, *this, InFlags, [](uint32 InA, uint32 InB) { return InA | InB; });
		return *this;
	}

	/**
	 * Return the bitwise XOR of two bit arrays. The resulting bit array will be sized according to InFlags.
	 */
	template <typename AllocatorA, typename AllocatorB>
	[[nodiscard]] static TBitArray BitwiseXOR(const TBitArray<AllocatorA>& A, const TBitArray<AllocatorB>& B, EBitwiseOperatorFlags InFlags)
	{
		TBitArray Result;
		UE::Core::Private::BitwiseBinaryOperatorImpl(A, B, Result, InFlags, [](uint32 InA, uint32 InB) { return InA ^ InB; });
		return Result;
	}

	/**
	 * Return the bitwise XOR of two bit arrays. The resulting bit array will be sized according to InFlags.
	 */
	template <typename OtherAllocator>
	TBitArray& CombineWithBitwiseXOR(const TBitArray<OtherAllocator>& InOther, EBitwiseOperatorFlags InFlags)
	{
		UE::Core::Private::BitwiseOperatorImpl(InOther, *this, InFlags, [](uint32 InA, uint32 InB) { return InA ^ InB; });
		return *this;
	}

	/**
	 * Perform a bitwise NOT on all the bits in this array
	 */
	void BitwiseNOT()
	{
		for (FWordIterator It(*this); It; ++It)
		{
			It.SetWord(~It.GetWord());
		}
	}

	/**
	 * Count the number of set bits in this array  FromIndex <= bit < ToIndex
	 */
	[[nodiscard]] int32 CountSetBits(int32 FromIndex = 0, int32 ToIndex = INDEX_NONE) const
	{
		if (ToIndex == INDEX_NONE)
		{
			ToIndex = NumBits;
		}

		checkSlow(FromIndex >= 0);
		checkSlow(ToIndex >= FromIndex && ToIndex <= NumBits);

		int32 NumSetBits = 0;
		for (FConstWordIterator It(*this, FromIndex, ToIndex); It; ++It)
		{
			NumSetBits += FMath::CountBits(It.GetWord());
		}
		return NumSetBits;
	}

	/**
	 * Returns true if Other contains all the same set bits as this, accounting for differences in length.
	 * Similar to operator== but can handle different length arrays by zero or one-filling missing bits.
	 *
	 * @param Other The array to compare against
	 * @param bMissingBitValue The value to use for missing bits when considering bits that are outside the range of either array
	 * @return true if this array matches Other, including any missing bits, false otherwise
	 */
	template <typename OtherAllocator>
	[[nodiscard]] bool CompareSetBits(const TBitArray<OtherAllocator>& Other, const bool bMissingBitValue) const
	{
		const uint32 MissingBitsFill = bMissingBitValue ? ~0u : 0;

		FConstWordIterator ThisIterator(*this);
		typename TBitArray<OtherAllocator>::FConstWordIterator OtherIterator(Other);

		ThisIterator.FillMissingBits(MissingBitsFill);
		OtherIterator.FillMissingBits(MissingBitsFill);

		while (ThisIterator || OtherIterator)
		{
			const uint32 A = ThisIterator  ? ThisIterator.GetWord()  : MissingBitsFill;
			const uint32 B = OtherIterator ? OtherIterator.GetWord() : MissingBitsFill;
			if (A != B)
			{
				return false;
			}

			++ThisIterator;
			++OtherIterator;
		}

		return true;
	}

	/**
	 * Pad this bit array with the specified value to ensure that it is at least the specified length. Does nothing if Num() >= DesiredNum.
	 * 
	 * @param DesiredNum The desired number of elements that should exist in the array.
	 * @param bPadValue  The value to pad with (0 or 1)
	 * @return The number of bits that were added to the array, or 0 if Num() >= DesiredNum.
	 */
	int32 PadToNum(int32 DesiredNum, bool bPadValue)
	{
		const int32 NumToAdd = DesiredNum - Num();
		if (NumToAdd > 0)
		{
			Add(bPadValue, NumToAdd);
			return NumToAdd;
		}
		return 0;
	}

	// Accessors.
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValidIndex(int32 InIndex) const
	{
		return InIndex >= 0 && InIndex < NumBits;
	}

	/**
	 * Returns true if the bit array is empty and contains no elements. 
	 *
	 * @returns True if the bit array is empty.
	 * @see Num
	 */
	[[nodiscard]] bool IsEmpty() const 
	{ 
		return NumBits == 0; 
	}

	[[nodiscard]] UE_FORCEINLINE_HINT int32 Num() const { return NumBits; }
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Max() const { return MaxBits; }
	[[nodiscard]] UE_FORCEINLINE_HINT FBitReference operator[](int32 Index)
	{
		check(Index>=0 && Index<NumBits);
		return FBitReference(
			GetData()[Index / NumBitsPerDWORD],
			1 << (Index & (NumBitsPerDWORD - 1))
			);
	}
	[[nodiscard]] UE_FORCEINLINE_HINT const FConstBitReference operator[](int32 Index) const
	{
		check(Index>=0 && Index<NumBits);
		return FConstBitReference(
			GetData()[Index / NumBitsPerDWORD],
			1 << (Index & (NumBitsPerDWORD - 1))
			);
	}
	[[nodiscard]] UE_FORCEINLINE_HINT FBitReference AccessCorrespondingBit(const FRelativeBitReference& RelativeReference)
	{
		checkSlow(RelativeReference.Mask);
		checkSlow(RelativeReference.WordIndex >= 0);
		checkSlow(((uint32)RelativeReference.WordIndex + 1) * NumBitsPerDWORD - 1 - FMath::CountLeadingZeros(RelativeReference.Mask) < (uint32)NumBits);
		return FBitReference(
			GetData()[RelativeReference.WordIndex],
			RelativeReference.Mask
			);
	}
	[[nodiscard]] UE_FORCEINLINE_HINT const FConstBitReference AccessCorrespondingBit(const FRelativeBitReference& RelativeReference) const
	{
		checkSlow(RelativeReference.Mask);
		checkSlow(RelativeReference.WordIndex >= 0);
		checkSlow(((uint32)RelativeReference.WordIndex + 1) * NumBitsPerDWORD - 1 - FMath::CountLeadingZeros(RelativeReference.Mask) < (uint32)NumBits);
		return FConstBitReference(
			GetData()[RelativeReference.WordIndex],
			RelativeReference.Mask
			);
	}

	/** BitArray iterator. */
	class FIterator : public FRelativeBitReference
	{
	public:
		[[nodiscard]] UE_FORCEINLINE_HINT explicit FIterator(TBitArray<Allocator>& InArray UE_LIFETIMEBOUND,int32 StartIndex = 0)
		:	FRelativeBitReference(StartIndex)
		,	Array(InArray)
		,	Index(StartIndex)
		{
		}
		UE_FORCEINLINE_HINT FIterator& operator++()
		{
			++Index;
			this->Mask <<= 1;
			if(!this->Mask)
			{
				// Advance to the next uint32.
				this->Mask = 1;
				++this->WordIndex;
			}
			return *this;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT FBitReference operator*() const
		{
			return GetValue();
		}

		/** conversion to "bool" returning true if the iterator is valid. */
		[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
		{ 
			return Index < Array.Num(); 
		}

		/** inverse of the "bool" operator */
		[[nodiscard]] UE_FORCEINLINE_HINT bool operator !() const
		{
			return !(bool)*this;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT FBitReference GetValue() const { return FBitReference(Array.GetData()[this->WordIndex],this->Mask); }
		[[nodiscard]] UE_FORCEINLINE_HINT int32 GetIndex() const { return Index; }
	private:
		TBitArray<Allocator>& Array;
		int32 Index;
	};

	/** Reverse iterator. */
	class FReverseIterator : public FRelativeBitReference
	{
	public:
		[[nodiscard]] UE_FORCEINLINE_HINT explicit FReverseIterator(TBitArray<Allocator>& InArray UE_LIFETIMEBOUND)
			: FRelativeBitReference(InArray.Num() - 1)
			, Array(InArray)
			, Index(InArray.Num() - 1)
		{
		}

		[[nodiscard]] UE_FORCEINLINE_HINT explicit FReverseIterator(TBitArray<Allocator>& InArray UE_LIFETIMEBOUND, int32 StartIndex)
			: FRelativeBitReference(-1)
			, Array(InArray)
			, Index(-1)
		{
		}

		UE_FORCEINLINE_HINT FReverseIterator& operator++()
		{
			--Index;
			this->Mask >>= 1;
			if (!this->Mask)
			{
				// Advance to the next uint32.
				this->Mask = (1u << (NumBitsPerDWORD - 1u));
				--this->WordIndex;
			}
			return *this;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT FBitReference operator*() const
		{
			return GetValue();
		}

		/** conversion to "bool" returning true if the iterator is valid. */
		[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
		{
			return Index >= 0;
		}

		/** inverse of the "bool" operator */
		[[nodiscard]] UE_FORCEINLINE_HINT bool operator !() const
		{
			return !(bool)*this;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT FBitReference GetValue() const { return FBitReference(Array.GetData()[this->WordIndex], this->Mask); }
		[[nodiscard]] UE_FORCEINLINE_HINT int32 GetIndex() const { return Index; }
	private:
		TBitArray<Allocator>& Array;
		int32 Index;
	};

	/** Const BitArray iterator. */
	class FConstIterator : public FRelativeBitReference
	{
	public:
		[[nodiscard]] UE_FORCEINLINE_HINT explicit FConstIterator(const TBitArray<Allocator>& InArray UE_LIFETIMEBOUND,int32 StartIndex = 0)
		:	FRelativeBitReference(StartIndex)
		,	Array(InArray)
		,	Index(StartIndex)
		{
		}
		UE_FORCEINLINE_HINT FConstIterator& operator++()
		{
			++Index;
			this->Mask <<= 1;
			if(!this->Mask)
			{
				// Advance to the next uint32.
				this->Mask = 1;
				++this->WordIndex;
			}
			return *this;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT FConstBitReference operator*() const
		{
			return GetValue();
		}

		/** conversion to "bool" returning true if the iterator is valid. */
		[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
		{
			return Index < Array.Num(); 
		}
		/** inverse of the "bool" operator */
		[[nodiscard]] UE_FORCEINLINE_HINT bool operator !() const 
		{
			return !(bool)*this;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT FConstBitReference GetValue() const { return FConstBitReference(Array.GetData()[this->WordIndex],this->Mask); }
		[[nodiscard]] UE_FORCEINLINE_HINT int32 GetIndex() const { return Index; }
	private:
		const TBitArray<Allocator>& Array;
		int32 Index;
	};

	/** Const reverse iterator. */
	class FConstReverseIterator : public FRelativeBitReference
	{
	public:
		[[nodiscard]] UE_FORCEINLINE_HINT explicit FConstReverseIterator(const TBitArray<Allocator>& InArray UE_LIFETIMEBOUND)
			:	FRelativeBitReference(InArray.Num() - 1)
			,	Array(InArray)
			,	Index(InArray.Num() - 1)
		{
		}
		[[nodiscard]] UE_FORCEINLINE_HINT explicit FConstReverseIterator(const TBitArray<Allocator>& InArray UE_LIFETIMEBOUND, int32 StartIndex)
			: FRelativeBitReference(-1)
			, Array(InArray)
			, Index(-1)
		{
		}
		UE_FORCEINLINE_HINT FConstReverseIterator& operator++()
		{
			--Index;
			this->Mask >>= 1u;
			if(!this->Mask)
			{
				// Advance to the next uint32.
				this->Mask = (1u << (NumBitsPerDWORD-1u));
				--this->WordIndex;
			}
			return *this;
		}

		UE_FORCEINLINE_HINT FConstBitReference operator*() const
		{
			return GetValue();
		}

		/** conversion to "bool" returning true if the iterator is valid. */
		[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
		{
			return Index >= 0; 
		}
		/** inverse of the "bool" operator */
		[[nodiscard]] UE_FORCEINLINE_HINT bool operator !() const 
		{
			return !(bool)*this;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT FConstBitReference GetValue() const { return FConstBitReference(Array.GetData()[this->WordIndex],this->Mask); }
		[[nodiscard]] UE_FORCEINLINE_HINT int32 GetIndex() const { return Index; }
	private:
		const TBitArray<Allocator>& Array;
		int32 Index;
	};

	[[nodiscard]] UE_FORCEINLINE_HINT const uint32* GetData() const
	{
		return (uint32*)AllocatorInstance.GetAllocation();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT uint32* GetData()
	{
		return (uint32*)AllocatorInstance.GetAllocation();
	}

private:
	template<typename WordType>
	struct TWordIteratorBase
	{
		[[nodiscard]] explicit operator bool() const
		{
			return CurrentIndex < NumWords;
		}

		[[nodiscard]] int32 GetIndex() const
		{
			return CurrentIndex;
		}

		[[nodiscard]] uint32 GetWord() const
		{
			checkSlow(CurrentIndex < NumWords);

			if (CurrentMask == ~0u)
			{
				return Data[CurrentIndex];
			}
			else if (MissingBitsFill == 0)
			{
				return Data[CurrentIndex] & CurrentMask;
			}
			else
			{
				return (Data[CurrentIndex] & CurrentMask) | (MissingBitsFill & ~CurrentMask);
			}
		}

		void operator++()
		{
			++this->CurrentIndex;
			if (this->CurrentIndex == NumWords-1)
			{
				CurrentMask = FinalMask;
			}
			else
			{
				CurrentMask = ~0u;
			}
		}

		void FillMissingBits(uint32 InMissingBitsFill)
		{
			MissingBitsFill = InMissingBitsFill;
		}

	protected:

		[[nodiscard]] explicit TWordIteratorBase(WordType* InData, int32 InStartBitIndex, int32 InEndBitIndex)
			: Data(InData)
			, CurrentIndex(InStartBitIndex / NumBitsPerDWORD)
			, NumWords(FMath::DivideAndRoundUp(InEndBitIndex, NumBitsPerDWORD))
			, CurrentMask(~0u << (InStartBitIndex % NumBitsPerDWORD))
			, FinalMask(~0u)
			, MissingBitsFill(0)
		{
			const int32 Shift = NumBitsPerDWORD - (InEndBitIndex % NumBitsPerDWORD);
			if (Shift < NumBitsPerDWORD)
			{
				FinalMask = ~0u >> Shift;
			}

			if (CurrentIndex == NumWords - 1)
			{
				CurrentMask &= FinalMask;
				FinalMask = CurrentMask;
			}
		}

		WordType* RESTRICT Data;

		int32 CurrentIndex;
		int32 NumWords;

		uint32 CurrentMask;
		uint32 FinalMask;
		uint32 MissingBitsFill;
	};

public:
	struct FConstWordIterator : TWordIteratorBase<const uint32>
	{
		[[nodiscard]] explicit FConstWordIterator(const TBitArray<Allocator>& InArray UE_LIFETIMEBOUND)
			: TWordIteratorBase<const uint32>(InArray.GetData(), 0, InArray.Num())
		{
		}

		[[nodiscard]] explicit FConstWordIterator(const TBitArray<Allocator>& InArray UE_LIFETIMEBOUND, int32 InStartBitIndex, int32 InEndBitIndex)
			: TWordIteratorBase<const uint32>(InArray.GetData(), InStartBitIndex, InEndBitIndex)
		{
			checkSlow(InStartBitIndex <= InEndBitIndex && InStartBitIndex <= InArray.Num() && InEndBitIndex <= InArray.Num());
			checkSlow(InStartBitIndex >= 0 && InEndBitIndex >= 0);
		}
	};


	struct FWordIterator : TWordIteratorBase<uint32>
	{
		[[nodiscard]] explicit FWordIterator(TBitArray<Allocator>& InArray UE_LIFETIMEBOUND)
			: TWordIteratorBase<uint32>(InArray.GetData(), 0, InArray.Num())
		{
		}

		void SetWord(uint32 InWord)
		{
			checkSlow(this->CurrentIndex < this->NumWords);

			if (this->CurrentIndex == this->NumWords-1)
			{
				this->Data[this->CurrentIndex] = InWord & this->FinalMask;
			}
			else
			{
				this->Data[this->CurrentIndex] = InWord;
			}
		}
	};

	/** Enables range-based for loops, DO NOT USE DIRECTLY. */

	[[nodiscard]] UE_FORCEINLINE_HINT FIterator      begin()       { return FIterator(*this); }
	[[nodiscard]] UE_FORCEINLINE_HINT FConstIterator begin() const { return FConstIterator(*this); }
	[[nodiscard]] UE_FORCEINLINE_HINT FIterator      end()         { return FIterator(*this, NumBits); }
	[[nodiscard]] UE_FORCEINLINE_HINT FConstIterator end()   const { return FConstIterator(*this, NumBits); }

	[[nodiscard]] UE_FORCEINLINE_HINT FReverseIterator      rbegin()       { return FReverseIterator(*this); }
	[[nodiscard]] UE_FORCEINLINE_HINT FConstReverseIterator rbegin() const { return FConstReverseIterator(*this); }
	[[nodiscard]] UE_FORCEINLINE_HINT FReverseIterator      rend()         { return FReverseIterator(*this, 0); }
	[[nodiscard]] UE_FORCEINLINE_HINT FConstReverseIterator rend()   const { return FConstReverseIterator(*this, 0); }

private:
	AllocatorType AllocatorInstance;
	int32         NumBits;
	int32         MaxBits;

	FORCENOINLINE void Realloc(int32 PreviousNumBits)
	{
		const uint32 PreviousNumWords = FBitSet::CalculateNumWords(PreviousNumBits);
		const uint32 MaxWords = FBitSet::CalculateNumWords(MaxBits);

		AllocatorInstance.ResizeAllocation(PreviousNumWords,MaxWords,sizeof(uint32));
		ClearPartialSlackBits(); // Implement class invariant
	}

	void SetBitNoCheck(int32 Index, bool Value)
	{
		uint32& Word = GetData()[Index/NumBitsPerDWORD];
		uint32 BitOffset = (Index % NumBitsPerDWORD);
		Word = (Word & ~(1 << BitOffset)) | (((uint32)Value) << BitOffset);
	}

	/**
	 * Clears the slack bits within the final partially relevant Word
	 */
	void ClearPartialSlackBits()
	{
		// TBitArray has a contract about bits outside of the active range - the bits in the final word past NumBits are guaranteed to be 0
		// This prevents easy-to-make determinism errors from users of TBitArray that do not carefully mask the final word.
		// It also allows us optimize some operations which would otherwise require us to mask the last word.
		const int32 UsedBits = NumBits % NumBitsPerDWORD;
		if (UsedBits != 0)
		{
			const int32  LastWordIndex = NumBits / NumBitsPerDWORD;
			const uint32 SlackMask = FullWordMask >> (NumBitsPerDWORD - UsedBits);

			uint32* LastWord = (GetData() + LastWordIndex);
			*LastWord = *LastWord & SlackMask;
		}
	}

	template<bool bFreezeMemoryImage, typename Dummy=void>
	struct TSupportsFreezeMemoryImageHelper
	{
		static void WriteMemoryImage(FMemoryImageWriter& Writer, const TBitArray&) { Writer.WriteBytes(TBitArray()); }
	};

	template<typename Dummy>
	struct TSupportsFreezeMemoryImageHelper<true, Dummy>
	{
		static void WriteMemoryImage(FMemoryImageWriter& Writer, const TBitArray& Object)
		{
			const int32 NumWords = FMath::DivideAndRoundUp(Object.NumBits, NumBitsPerDWORD);
			Object.AllocatorInstance.WriteMemoryImage(Writer, StaticGetTypeLayoutDesc<uint32>(), NumWords);
			Writer.WriteBytes(Object.NumBits);
			Writer.WriteBytes(Object.NumBits);
		}
	};

public:
	void WriteMemoryImage(FMemoryImageWriter& Writer) const
	{
		static constexpr bool bSupportsFreezeMemoryImage = TAllocatorTraits<Allocator>::SupportsFreezeMemoryImage;
		checkf(!Writer.Is32BitTarget(), TEXT("TBitArray does not currently support freezing for 32bits"));
		TSupportsFreezeMemoryImageHelper<bSupportsFreezeMemoryImage>::WriteMemoryImage(Writer, *this);
	}
};

namespace Freeze
{
	template<typename Allocator>
	void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const TBitArray<Allocator>& Object, const FTypeLayoutDesc&)
	{
		Object.WriteMemoryImage(Writer);
	}
}

DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT(template<typename Allocator>, TBitArray<Allocator>);

template<typename Allocator>
[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetTypeHash(const TBitArray<Allocator>& BitArray)
{
	uint32 NumWords = FBitSet::CalculateNumWords(BitArray.Num());
	uint32 Hash = NumWords;
	const uint32* Data = BitArray.GetData();
	for (uint32 i = 0; i < NumWords; i++)
	{
		Hash ^= Data[i];
	}
	return Hash;
}


/** An iterator which only iterates over set bits. */
template<typename Allocator>
class TConstSetBitIterator : public FRelativeBitReference
{
public:

	[[nodiscard]] explicit TConstSetBitIterator(const TBitArray<Allocator>& InArray UE_LIFETIMEBOUND)
		: FRelativeBitReference(0)
		, Array                (InArray)
		, UnvisitedBitMask     (~0U)
		, CurrentBitIndex      (0)
		, BaseBitIndex         (0)
	{
		if (Array.Num())
		{
			FindFirstSetBit();
		}
	}

	[[nodiscard]] explicit TConstSetBitIterator(const TBitArray<Allocator>& InArray UE_LIFETIMEBOUND, int32 StartIndex)
		: FRelativeBitReference(StartIndex)
		, Array                (InArray)
		, UnvisitedBitMask     ((~0U) << (StartIndex & (NumBitsPerDWORD - 1)))
		, CurrentBitIndex      (StartIndex)
		, BaseBitIndex         (StartIndex & ~(NumBitsPerDWORD - 1))
	{
		check(StartIndex >= 0 && StartIndex <= Array.Num());
		if (StartIndex != Array.Num())
		{
			FindFirstSetBit();
		}
	}

	/** Forwards iteration operator. */
	UE_FORCEINLINE_HINT TConstSetBitIterator& operator++()
	{
		// Mark the current bit as visited.
		UnvisitedBitMask &= ~this->Mask;

		// Find the first set bit that hasn't been visited yet.
		FindFirstSetBit();

		return *this;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TConstSetBitIterator& Rhs) const
	{
		// We only need to compare the bit index and the array... all the rest of the state is unobservable.
		return CurrentBitIndex == Rhs.CurrentBitIndex && &Array == &Rhs.Array;
	}

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TConstSetBitIterator& Rhs) const
	{ 
		return !(*this == Rhs);
	}
#endif

	/** conversion to "bool" returning true if the iterator is valid. */
	[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
	{ 
		return CurrentBitIndex < Array.Num(); 
	}
	/** inverse of the "bool" operator */
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator !() const
	{
		return !(bool)*this;
	}

	/** Index accessor. */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 GetIndex() const
	{
		return CurrentBitIndex;
	}

private:

	const TBitArray<Allocator>& Array;

	uint32 UnvisitedBitMask;
	int32 CurrentBitIndex;
	int32 BaseBitIndex;

	/** Find the first set bit starting with the current bit, inclusive. */
	void FindFirstSetBit()
	{
		const uint32* ArrayData      = Array.GetData();
		const int32   ArrayNum       = Array.Num();
		const int32   LastWordIndex = (ArrayNum - 1) / NumBitsPerDWORD;

		// Advance to the next non-zero uint32.
		uint32 RemainingBitMask = ArrayData[this->WordIndex] & UnvisitedBitMask;
		while (!RemainingBitMask)
		{
			++this->WordIndex;
			BaseBitIndex += NumBitsPerDWORD;
			if (this->WordIndex > LastWordIndex)
			{
				// We've advanced past the end of the array.
				CurrentBitIndex = ArrayNum;
				return;
			}

			RemainingBitMask = ArrayData[this->WordIndex];
			UnvisitedBitMask = ~0u;
		}

		// This operation has the effect of unsetting the lowest set bit of BitMask
		const uint32 NewRemainingBitMask = RemainingBitMask & (RemainingBitMask - 1);

		// This operation XORs the above mask with the original mask, which has the effect
		// of returning only the bits which differ; specifically, the lowest bit
		this->Mask = NewRemainingBitMask ^ RemainingBitMask;

		// If the Nth bit was the lowest set bit of BitMask, then this gives us N
		CurrentBitIndex = BaseBitIndex + NumBitsPerDWORD - 1 - FMath::CountLeadingZeros(this->Mask);

		// If we've accidentally iterated off the end of an array but still within the same Word
		// then set the index to the last index of the array
		if (CurrentBitIndex > ArrayNum)
		{
			CurrentBitIndex = ArrayNum;
		}
	}
};


/** An iterator which only iterates over the bits which are set in both of two bit-arrays, 
    if the Both template argument is true, or either if the argument is false. */
template<typename Allocator,typename OtherAllocator, bool Both>
class TConstDualSetBitIterator : public FRelativeBitReference
{
public:

	/** Constructor. */
	[[nodiscard]] UE_FORCEINLINE_HINT TConstDualSetBitIterator(
		const TBitArray<Allocator>& InArrayA,
		const TBitArray<OtherAllocator>& InArrayB,
		int32 StartIndex = 0
		)
	:	FRelativeBitReference(StartIndex)
	,	ArrayA(InArrayA)
	,	ArrayB(InArrayB)
	,	UnvisitedBitMask((~0U) << (StartIndex & (NumBitsPerDWORD - 1)))
	,	CurrentBitIndex(StartIndex)
	,	BaseBitIndex(StartIndex & ~(NumBitsPerDWORD - 1))
	{
		check(ArrayA.Num() == ArrayB.Num());

		FindFirstSetBit();
	}

	/** Advancement operator. */
	UE_FORCEINLINE_HINT TConstDualSetBitIterator& operator++()
	{
		checkSlow(ArrayA.Num() == ArrayB.Num());

		// Mark the current bit as visited.
		UnvisitedBitMask &= ~this->Mask;

		// Find the first set bit that hasn't been visited yet.
		FindFirstSetBit();

		return *this;

	}

	/** conversion to "bool" returning true if the iterator is valid. */
	[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
	{ 
		return CurrentBitIndex < ArrayA.Num(); 
	}
	/** inverse of the "bool" operator */
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator !() const
	{
		return !(bool)*this;
	}

	/** Index accessor. */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 GetIndex() const
	{
		return CurrentBitIndex;
	}

private:

	const TBitArray<Allocator>& ArrayA;
	const TBitArray<OtherAllocator>& ArrayB;

	uint32 UnvisitedBitMask;
	int32 CurrentBitIndex;
	int32 BaseBitIndex;

	/** Find the first bit that is set in both arrays, starting with the current bit, inclusive. */
	void FindFirstSetBit()
	{
		const uint32 EmptyArrayData = 0;
		const uint32* ArrayDataA = ArrayA.GetData();
		if (!ArrayDataA)
		{
			ArrayDataA = &EmptyArrayData;
		}
		const uint32* ArrayDataB = ArrayB.GetData();
		if (!ArrayDataB)
		{
			ArrayDataB = &EmptyArrayData;
		}

		// Advance to the next non-zero uint32.
		uint32 RemainingBitMask;
		
		if (Both)
		{
			RemainingBitMask = ArrayDataA[this->WordIndex] & ArrayDataB[this->WordIndex] & UnvisitedBitMask;
		}
		else
		{
			RemainingBitMask = (ArrayDataA[this->WordIndex] | ArrayDataB[this->WordIndex]) & UnvisitedBitMask;
		}

		while(!RemainingBitMask)
		{
			this->WordIndex++;
			BaseBitIndex += NumBitsPerDWORD;
			const int32 LastWordIndex = (ArrayA.Num() - 1) / NumBitsPerDWORD;
			if (this->WordIndex <= LastWordIndex)
			{
				if (Both)
				{
					RemainingBitMask = ArrayDataA[this->WordIndex] & ArrayDataB[this->WordIndex];
				}
				else
				{
					RemainingBitMask = ArrayDataA[this->WordIndex] | ArrayDataB[this->WordIndex];
				}

				UnvisitedBitMask = ~0;
			}
			else
			{
				// We've advanced past the end of the array.
				CurrentBitIndex = ArrayA.Num();
				return;
			}
		};

		// We can assume that RemainingBitMask!=0 here.
		checkSlow(RemainingBitMask);

		// This operation has the effect of unsetting the lowest set bit of BitMask
		const uint32 NewRemainingBitMask = RemainingBitMask & (RemainingBitMask - 1);

		// This operation XORs the above mask with the original mask, which has the effect
		// of returning only the bits which differ; specifically, the lowest bit
		this->Mask = NewRemainingBitMask ^ RemainingBitMask;

		// If the Nth bit was the lowest set bit of BitMask, then this gives us N
		CurrentBitIndex = BaseBitIndex + NumBitsPerDWORD - 1 - FMath::CountLeadingZeros(this->Mask);
	}
};

// A specialization of TConstDualSetBitIterator for iterating over two TBitArray containers and 
// stop only when bits in the same location in each are both set.
template<typename Allocator, typename OtherAllocator>
using TConstDualBothSetBitIterator = TConstDualSetBitIterator<Allocator, OtherAllocator, true>;

// A specialization of TConstDualSetBitIterator for iterating over two TBitArray containers and 
// stop only when either, or both, of the bits in the same location in each are set.
template<typename Allocator, typename OtherAllocator>
using TConstDualEitherSetBitIterator = TConstDualSetBitIterator<Allocator, OtherAllocator, false>;

// Untyped bit array type for accessing TBitArray data, like FScriptArray for TArray.
// Must have the same memory representation as a TBitArray.
template <typename Allocator, typename InDerivedType>
class TScriptBitArray
{
	using DerivedType = std::conditional_t<std::is_void_v<InDerivedType>, TScriptBitArray, InDerivedType>;

public:
	/**
	 * Minimal initialization constructor.
	 * @param Value - The value to initial the bits to.
	 * @param InNumBits - The initial number of bits in the array.
	 */
	[[nodiscard]] TScriptBitArray()
		: NumBits(0)
		, MaxBits(0)
	{
	}

	[[nodiscard]] bool IsValidIndex(int32 Index) const
	{
		return Index >= 0 && Index < NumBits;
	}

	[[nodiscard]] FBitReference operator[](int32 Index)
	{
		check(IsValidIndex(Index));
		return FBitReference(GetData()[Index / NumBitsPerDWORD], 1 << (Index & (NumBitsPerDWORD - 1)));
	}

	[[nodiscard]] FConstBitReference operator[](int32 Index) const
	{
		check(IsValidIndex(Index));
		return FConstBitReference(GetData()[Index / NumBitsPerDWORD], 1 << (Index & (NumBitsPerDWORD - 1)));
	}

	void MoveAssign(DerivedType& Other)
	{
		checkSlow(this != &Other);
		Empty(0);
		AllocatorInstance.MoveToEmpty(Other.AllocatorInstance);
		NumBits = Other.NumBits; Other.NumBits = 0;
		MaxBits = Other.MaxBits; Other.MaxBits = 0;
	}

	void Empty(int32 Slack = 0)
	{
		NumBits = 0;

		Slack = FBitSet::CalculateNumWords(Slack) * NumBitsPerDWORD;
		// If the expected number of bits doesn't match the allocated number of bits, reallocate.
		if (MaxBits != Slack)
		{
			MaxBits = Slack;
			Realloc(0);
		}
	}

	int32 Add(const bool Value)
	{
		const int32 Index = NumBits;
		NumBits++;
		if (NumBits > MaxBits)
		{
			ReallocGrow(NumBits - 1);
		}
		else if ((Index % NumBitsPerDWORD) == 0)
		{
			// Clear the new word to maintain the ClearPartialSlackBits invariant.
			GetData()[Index / NumBitsPerDWORD] = 0;
		}
		(*this)[Index] = Value;
		return Index;
	}

private:
	typedef typename Allocator::template ForElementType<uint32> AllocatorType;

	AllocatorType AllocatorInstance;
	int32         NumBits;
	int32         MaxBits;

	// This function isn't intended to be called, just to be compiled to validate the correctness of the type.
	static void CheckConstraints()
	{
		typedef TScriptBitArray ScriptType;
		typedef TBitArray<>     RealType;

		// Check that the class footprint is the same
		static_assert(sizeof (ScriptType) == sizeof (RealType), "TScriptBitArray's size doesn't match TBitArray");
		static_assert(alignof(ScriptType) == alignof(RealType), "TScriptBitArray's alignment doesn't match TBitArray");

		// Check member sizes
		static_assert(sizeof(DeclVal<ScriptType>().AllocatorInstance) == sizeof(DeclVal<RealType>().AllocatorInstance), "TScriptBitArray's AllocatorInstance member size does not match TBitArray's");
		static_assert(sizeof(DeclVal<ScriptType>().NumBits)           == sizeof(DeclVal<RealType>().NumBits),           "TScriptBitArray's NumBits member size does not match TBitArray's");
		static_assert(sizeof(DeclVal<ScriptType>().MaxBits)           == sizeof(DeclVal<RealType>().MaxBits),           "TScriptBitArray's MaxBits member size does not match TBitArray's");

		// Check member offsets
		static_assert(STRUCT_OFFSET(ScriptType, AllocatorInstance) == STRUCT_OFFSET(RealType, AllocatorInstance), "TScriptBitArray's AllocatorInstance member offset does not match TBitArray's");
		static_assert(STRUCT_OFFSET(ScriptType, NumBits)           == STRUCT_OFFSET(RealType, NumBits),           "TScriptBitArray's NumBits member offset does not match TBitArray's");
		static_assert(STRUCT_OFFSET(ScriptType, MaxBits)           == STRUCT_OFFSET(RealType, MaxBits),           "TScriptBitArray's MaxBits member offset does not match TBitArray's");
	}

	[[nodiscard]] UE_FORCEINLINE_HINT uint32* GetData()
	{
		return (uint32*)AllocatorInstance.GetAllocation();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT const uint32* GetData() const
	{
		return (const uint32*)AllocatorInstance.GetAllocation();
	}

	FORCENOINLINE void Realloc(int32 PreviousNumBits)
	{
		const uint32 MaxWords = AllocatorInstance.CalculateSlackReserve(
			FBitSet::CalculateNumWords(MaxBits),
			sizeof(uint32)
			);
		MaxBits = MaxWords * NumBitsPerDWORD;
		const uint32 PreviousNumWords = FBitSet::CalculateNumWords(PreviousNumBits);

		AllocatorInstance.ResizeAllocation(PreviousNumWords, MaxWords, sizeof(uint32));

		if (MaxWords && MaxWords > PreviousNumWords)
		{
			// Reset the newly allocated slack Words.
			FMemory::Memzero((uint32*)AllocatorInstance.GetAllocation() + PreviousNumWords, (MaxWords - PreviousNumWords) * sizeof(uint32));
		}
	}
	FORCENOINLINE void ReallocGrow(int32 PreviousNumBits)
	{
		// Allocate memory for the new bits.
		const uint32 MaxWords = AllocatorInstance.CalculateSlackGrow(
			FBitSet::CalculateNumWords(NumBits),
			FBitSet::CalculateNumWords(MaxBits),
			sizeof(uint32)
			);
		MaxBits = MaxWords * NumBitsPerDWORD;
		const uint32 PreviousNumWords = FBitSet::CalculateNumWords(PreviousNumBits);
		AllocatorInstance.ResizeAllocation(PreviousNumWords, MaxWords, sizeof(uint32));
		if (MaxWords && MaxWords > PreviousNumWords)
		{
			// Reset the newly allocated slack Words.
			FMemory::Memzero((uint32*)AllocatorInstance.GetAllocation() + PreviousNumWords, (MaxWords - PreviousNumWords) * sizeof(uint32));
		}
	}

public:
	// These should really be private, because they shouldn't be called, but there's a bunch of code
	// that needs to be fixed first.
	[[nodiscard]] TScriptBitArray(const TScriptBitArray&) { check(false); }
	void operator=(const TScriptBitArray&) { check(false); }
};

template <typename AllocatorType, typename InDerivedType>
struct TIsZeroConstructType<TScriptBitArray<AllocatorType, InDerivedType>>
{
	enum { Value = true };
};

class FScriptBitArray : public TScriptBitArray<FDefaultBitArrayAllocator, FScriptBitArray>
{
	using Super = TScriptBitArray<FDefaultBitArrayAllocator, FScriptBitArray>;

public:
	using Super::Super;
};

template<typename Allocator>
FArchive& operator<<(FArchive& Ar, TBitArray<Allocator>& BitArray)
{
	BitArray.Serialize(Ar);
	return Ar;
}
