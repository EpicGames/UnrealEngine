// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/IntrusiveUnsetOptionalState.h"
#include "Templates/UnrealTypeTraits.h"
#include "Containers/IndirectArray.h"
#include "Traits/IsTriviallyRelocatable.h"

namespace UE4ChunkedArray_Private
{
	template <typename ChunkType, typename ElementType, uint32 NumElementsPerChunk>
	struct TChunkedArrayIterator
	{
		ChunkType**	Chunk;
		uint32		Count = 0;
		uint32		ElementIndex = 0;

		[[nodiscard]] ElementType& operator*() const
		{
			return (*Chunk)->Elements[ElementIndex];
		}

		void operator++()
		{
			++ElementIndex;
			if (ElementIndex >= NumElementsPerChunk)
			{
				ElementIndex = 0;
				++Chunk;
			}

			++Count;
		}

		[[nodiscard]] bool operator!=(const TChunkedArrayIterator& Rhs) const
		{
			return Count < Rhs.Count;
		}
	};
}

// Forward declarations

template<typename InElementType, uint32 TargetBytesPerChunk, typename AllocatorType>
class TChunkedArray;

template <typename T, uint32 TargetBytesPerChunk, typename AllocatorType>
void* operator new(size_t Size, TChunkedArray<T, TargetBytesPerChunk, AllocatorType>& ChunkedArray);

/** An array that uses multiple allocations to avoid allocation failure due to fragmentation. */
template<typename InElementType, uint32 TargetBytesPerChunk = 16384, typename AllocatorType = FDefaultAllocator >
class TChunkedArray
{
	using ElementType = InElementType;

public:

	/** Initialization constructor. */
	[[nodiscard]] TChunkedArray(int32 InNumElements = 0):
		NumElements(InNumElements)
	{
		// Compute the number of chunks needed.
		const int32 NumChunks = (NumElements + NumElementsPerChunk - 1) / NumElementsPerChunk;

		// Allocate the chunks.
		Chunks.Empty(NumChunks);
		for(int32 ChunkIndex = 0;ChunkIndex < NumChunks;ChunkIndex++)
		{
			Chunks.Add(new FChunk);
		}
	}

private:
	template <typename ArrayType>
	inline static void Move(ArrayType& ToArray, ArrayType& FromArray)
	{
		ToArray.Chunks      = (ChunksType&&)FromArray.Chunks;
		ToArray.NumElements = FromArray.NumElements;
		FromArray.NumElements = 0;
	}

public:
	[[nodiscard]] TChunkedArray(TChunkedArray&& Other)
	{
		this->Move(*this, Other);
	}

	TChunkedArray& operator=(TChunkedArray&& Other)
	{
		if (this != &Other)
		{
			this->Move(*this, Other);
		}

		return *this;
	}

	~TChunkedArray()
	{
		UE_STATIC_ASSERT_WARN(TIsTriviallyRelocatable_V<InElementType>, "TChunkedArray can only be used with trivially relocatable types");
	}

	[[nodiscard]] TChunkedArray(const TChunkedArray&) = default;
	TChunkedArray& operator=(const TChunkedArray&) = default;

	//////////////////////////////////////////////////////
	// Start - intrusive TOptional<TChunkedArray> state //
	//////////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TChunkedArray;

	[[nodiscard]] explicit TChunkedArray(FIntrusiveUnsetOptionalState)
		: NumElements(-1)
	{
	}
	[[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState) const
	{
		return NumElements == -1;
	}
	////////////////////////////////////////////////////
	// End - intrusive TOptional<TChunkedArray> state //
	////////////////////////////////////////////////////

	// Accessors.
	[[nodiscard]] ElementType& operator()(int32 ElementIndex)
	{
		const uint32 ChunkIndex = ElementIndex / NumElementsPerChunk;
		const uint32 ChunkElementIndex = ElementIndex % NumElementsPerChunk;
		return Chunks[ChunkIndex].Elements[ChunkElementIndex];
	}
	[[nodiscard]] const ElementType& operator()(int32 ElementIndex) const
	{
		const int32 ChunkIndex = ElementIndex / NumElementsPerChunk;
		const int32 ChunkElementIndex = ElementIndex % NumElementsPerChunk;
		return Chunks[ChunkIndex].Elements[ChunkElementIndex];
	}
	[[nodiscard]] ElementType& operator[](int32 ElementIndex)
	{
		const uint32 ChunkIndex = ElementIndex / NumElementsPerChunk;
		const uint32 ChunkElementIndex = ElementIndex % NumElementsPerChunk;
		return Chunks[ChunkIndex].Elements[ChunkElementIndex];
	}
	[[nodiscard]] const ElementType& operator[](int32 ElementIndex) const
	{
		const int32 ChunkIndex = ElementIndex / NumElementsPerChunk;
		const int32 ChunkElementIndex = ElementIndex % NumElementsPerChunk;
		return Chunks[ChunkIndex].Elements[ChunkElementIndex];
	}

	/**
	 * Returns true if the chunked array is empty and contains no elements. 
	 *
	 * @returns True if the chunked array is empty.
	 * @see Num
	 */
	[[nodiscard]] bool IsEmpty() const
	{
		return NumElements == 0;
	}

	[[nodiscard]] int32 Num() const
	{ 
		return NumElements; 
	}

	[[nodiscard]] SIZE_T GetAllocatedSize( void ) const
	{
		return Chunks.GetAllocatedSize();
	}

	/**
	* Tests if index is valid, i.e. greater than zero and less than number of
	* elements in array.
	*
	* @param Index Index to test.
	*
	* @returns True if index is valid. False otherwise.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValidIndex(int32 Index) const
	{
		return Index >= 0 && Index < NumElements;
	}

	/**
	 * Adds a new item to the end of the chunked array.
	 *
	 * @param Item	The item to add
	 * @return		Index to the new item
	 */
	int32 AddElement(ElementType&& Item)
	{
		new(*this) ElementType(MoveTempIfPossible(Item));
		return this->NumElements - 1;
	}

	/**
	 * Adds a new item to the end of the chunked array.
	 *
	 * @param Item	The item to add
	 * @return		Index to the new item
	 */
	int32 AddElement( const ElementType& Item )
	{
		new(*this) ElementType(Item);
		return this->NumElements - 1;
	}

	/**
	 * Constructs a new item to the end of the chunked array.
	 *
	 * @param Args	The arguments to forward to the constructor of the new item.
	 * @return		Index to the new item
	 */
	template <typename... ArgsType>
	int32 Emplace(ArgsType&&... Args)
	{
		new(*this) ElementType(Forward<ArgsType>(Args)...);
		return this->NumElements - 1;
	}

	/**
	 * Appends the specified array to this array.
	 * Cannot append to self.
	 *
	 * @param Other The array to append.
	 */
	inline TChunkedArray& operator+=(const TArray<ElementType>& Other) 
	{ 
		if( (UPTRINT*)this != (UPTRINT*)&Other )
		{
			for( const auto& It : Other )
			{
				AddElement(It);
			}
		}
		return *this; 
	}

	inline TChunkedArray& operator+=(const TChunkedArray& Other) 
	{ 
		if( (UPTRINT*)this != (UPTRINT*)&Other )
		{
			for( int32 Index = 0; Index < Other.Num(); ++Index )
			{
				AddElement(Other[Index]);
			}
		}
		return *this; 
	}

	int32 Add( int32 Count=1 )
	{
		check(Count>=0);
		checkSlow(NumElements>=0);

		const int32 OldNum = NumElements;
		const int32 NewNumElements = OldNum + Count;
		const int32 NewNumChunks = (NewNumElements + NumElementsPerChunk - 1)/NumElementsPerChunk;
		NumElements = NewNumElements;
		for (int32 NumChunks = Chunks.Num(); NumChunks < NewNumChunks; ++NumChunks)
		{
			Chunks.Add(new FChunk);
		}

		return OldNum;
	}

	template<typename OtherAllocator>
	void CopyToLinearArray(TArray<ElementType, OtherAllocator>& DestinationArray)
	{
		if (NumElements > 0)
		{
			int32 OriginalNumElements = DestinationArray.Num();
			DestinationArray.AddUninitialized(NumElements);
			InElementType* CopyDestPtr = &DestinationArray[OriginalNumElements];

			for (int32 ChunkIndex = 0; ChunkIndex < Chunks.Num(); ChunkIndex++)
			{
				const int32 NumElementsInCurrentChunk = FMath::Min<int32>(NumElements - ChunkIndex * NumElementsPerChunk, NumElementsPerChunk);
				check(NumElementsInCurrentChunk > 0);
				ConstructItems<ElementType>(CopyDestPtr, &Chunks[ChunkIndex].Elements[0], NumElementsInCurrentChunk);
				CopyDestPtr += NumElementsInCurrentChunk;
			}
		}
	}

	template<typename OtherAllocator>
	void MoveToLinearArray(TArray<ElementType, OtherAllocator>& DestinationArray)
	{
		if (NumElements > 0)
		{
			int32 OriginalNumElements = DestinationArray.Num();
			DestinationArray.AddUninitialized(NumElements);
			InElementType* CopyDestPtr = &DestinationArray[OriginalNumElements];

			for (int32 ChunkIndex = 0; ChunkIndex < Chunks.Num(); ChunkIndex++)
			{
				const int32 NumElementsInCurrentChunk = FMath::Min<int32>(NumElements - ChunkIndex * NumElementsPerChunk, NumElementsPerChunk);
				check(NumElementsInCurrentChunk > 0);
				MoveConstructItems<ElementType>(CopyDestPtr, &Chunks[ChunkIndex].Elements[0], NumElementsInCurrentChunk);
				CopyDestPtr += NumElementsInCurrentChunk;
			}

			Empty();
		}
	}

	void Empty( int32 Slack=0 ) 
	{
		// Compute the number of chunks needed.
		const int32 NumChunks = (Slack + NumElementsPerChunk - 1) / NumElementsPerChunk;
		Chunks.Empty(NumChunks);
		NumElements = 0;
	}

	/**
	 * Reserves memory such that the array can contain at least Number elements.
	 *
	 * @param Number The number of elements that the array should be able to
	 *               contain after allocation.
	 */
	void Reserve(int32 Number)
	{
		// Compute the number of chunks needed.
		const int32 NumChunks = (Number + NumElementsPerChunk - 1) / NumElementsPerChunk;
		Chunks.Reserve(NumChunks);
	}

	void Shrink()
	{
		Chunks.Shrink();
	}

protected:

	enum { NumElementsPerChunk = TargetBytesPerChunk / sizeof(ElementType) };

	/** A chunk of the array's elements. */
	struct FChunk
	{
		/** The elements in the chunk. */
		ElementType Elements[NumElementsPerChunk];
	};

	/** The chunks of the array's elements. */
	typedef TIndirectArray<FChunk, AllocatorType> ChunksType;
	ChunksType Chunks;

	/** The number of elements in the array. */
	int32 NumElements;

private:
	typedef UE4ChunkedArray_Private::TChunkedArrayIterator<      FChunk,       ElementType, NumElementsPerChunk> FIterType;
	typedef UE4ChunkedArray_Private::TChunkedArrayIterator<const FChunk, const ElementType, NumElementsPerChunk> FConstIterType;

public:
	[[nodiscard]] FIterType begin()
	{
		return FIterType{Chunks.GetData()};
	}

	[[nodiscard]] FConstIterType begin() const
	{
		return FConstIterType{Chunks.GetData()};
	}

	[[nodiscard]] FIterType end()
	{
		return FIterType{nullptr, uint32(NumElements)};
	}

	[[nodiscard]] FConstIterType end() const
	{
		return FConstIterType{nullptr, uint32(NumElements)};
	}
};



template <typename T,uint32 TargetBytesPerChunk, typename AllocatorType> 
void* operator new( size_t Size, TChunkedArray<T,TargetBytesPerChunk, AllocatorType>& ChunkedArray )
{
	check(Size == sizeof(T));
	const int32 Index = ChunkedArray.Add(1);
	return &ChunkedArray(Index);
}
