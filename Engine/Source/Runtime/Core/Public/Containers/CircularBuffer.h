// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/Array.h"
#include "Math/UnrealMathUtility.h"

/**
 * Template for circular buffers.
 *
 * The size of the buffer is rounded up to the next power of two in order speed up indexing
 * operations using a simple bit mask instead of the commonly used modulus operator that may
 * be slow on some platforms.
 */
template<typename InElementType> class TCircularBuffer
{
public:
	using ElementType = InElementType;

	/**
	 * Creates and initializes a new instance of the TCircularBuffer class.
	 *
	 * @param Capacity The number of elements that the buffer can store (will be rounded up to the next power of 2).
	 */
	[[nodiscard]] explicit TCircularBuffer(uint32 Capacity)
	{
		checkSlow(Capacity <= 0x40000000U);	// Limited by powers-of-2 counts that can fit into a TArray

		Elements.AddZeroed(FMath::RoundUpToPowerOfTwo(Capacity));
		IndexMask = Elements.Num() - 1;
	}

	/**
	 * Creates and initializes a new instance of the TCircularBuffer class.
	 *
	 * @param Capacity The number of elements that the buffer can store (will be rounded up to the next power of 2).
	 * @param InitialValue The initial value for the buffer's elements.
	 */
	[[nodiscard]] TCircularBuffer(uint32 Capacity, const ElementType& InitialValue)
	{
		checkSlow(Capacity <= 0x40000000U); // Limited by powers-of-2 counts that can fit into a TArray

		Elements.Init(InitialValue, FMath::RoundUpToPowerOfTwo(Capacity));
		IndexMask = Elements.Num() - 1;
	}

public:

	/**
	 * Returns the mutable element at the specified index.
	 *
	 * @param Index The index of the element to return.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& operator[](uint32 Index)
	{
		return Elements[Index & IndexMask];
	}

	/**
	 * Returns the immutable element at the specified index.
	 *
	 * @param Index The index of the element to return.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType& operator[](uint32 Index) const
	{
		return Elements[Index & IndexMask];
	}
	
public:

	/**
	 * Returns the number of elements that the buffer can hold.
	 *
	 * @return Buffer capacity.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT uint32 Capacity() const
	{
		return Elements.Num();
	}

	/**
	 * Calculates the index that follows the given index.
	 *
	 * @param CurrentIndex The current index.
	 * @return The next index.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetNextIndex(uint32 CurrentIndex) const
	{
		return ((CurrentIndex + 1) & IndexMask);
	}

	/**
	 * Calculates the index previous to the given index.
	 *
	 * @param CurrentIndex The current index.
	 * @return The previous index.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetPreviousIndex(uint32 CurrentIndex) const
	{
		return ((CurrentIndex - 1) & IndexMask);
	}

private:

	/** Holds the mask for indexing the buffer's elements. */
	uint32 IndexMask;

	/** Holds the buffer's elements. */
	TArray<ElementType> Elements;
};
