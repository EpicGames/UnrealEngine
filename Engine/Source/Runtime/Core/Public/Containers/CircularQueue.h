// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/CircularBuffer.h"
#include "Templates/Atomic.h"

// WARNING: This queue is planned for deprecation in favor of TSpscQueue

/**
 * Implements a lock-free first-in first-out queue using a circular array.
 *
 * This class is thread safe only in single-producer single-consumer scenarios.
 *
 * The number of items that can be enqueued is one less than the queue's capacity,
 * because one item will be used for detecting full and empty states.
 *
 * There is some room for optimization via using fine grained memory fences, but
 * the implications for all of our target platforms need further analysis, so
 * we're using the simpler sequentially consistent model for now.
 *
 * @param T The type of elements held in the queue.
 */
template<typename T> class TCircularQueue
{
public:
	using FElementType = T;

	/**
	 * Constructor.
	 *
	 * @param CapacityPlusOne The number of elements that the queue can hold (will be rounded up to the next power of 2).
	 */
	explicit TCircularQueue(uint32 CapacityPlusOne)
		: Buffer(CapacityPlusOne)
		, Head(0)
		, Tail(0)
	{ }

public:

	/**
	 * Gets the number of elements in the queue.
	 *
	 * Can be called from any thread. The result reflects the calling thread's current
	 * view. Since no locking is used, different threads may return different results.
	 *
	 * @return Number of queued elements.
	 */
	uint32 Count() const
	{
		int32 Count = Tail.Load() - Head.Load();

		if (Count < 0)
		{
			Count += Buffer.Capacity();
		}

		return (uint32)Count;
	}

	/**
	 * Removes an item from the front of the queue.
	 *
	 * @param OutElement Will contain the element if the queue is not empty.
	 * @return true if an element has been returned, false if the queue was empty.
	 * @note To be called only from consumer thread.
	 */
	bool Dequeue(FElementType& OutElement)
	{
		const uint32 CurrentHead = Head.Load();

		if (CurrentHead != Tail.Load())
		{
			OutElement = MoveTemp(Buffer[CurrentHead]);
			Head.Store(Buffer.GetNextIndex(CurrentHead));

			return true;
		}

		return false;
	}

	/**
	 * Removes an item from the front of the queue.
	 *
	 * @return true if an element has been removed, false if the queue was empty.
	 * @note To be called only from consumer thread.
	 */
	bool Dequeue()
	{
		const uint32 CurrentHead = Head.Load();

		if (CurrentHead != Tail.Load())
		{
			Head.Store(Buffer.GetNextIndex(CurrentHead));

			return true;
		}

		return false;
	}

	/**
	 * Empties the queue.
	 *
	 * @note To be called only from consumer thread.
	 * @see IsEmpty
	 */
	void Empty()
	{
		Head.Store(Tail.Load());
	}

	/**
	 * Adds an item to the end of the queue.
	 *
	 * @param Element The element to add.
	 * @return true if the item was added, false if the queue was full.
	 * @note To be called only from producer thread.
	 */
	bool Enqueue(const FElementType& Element)
	{
		const uint32 CurrentTail = Tail.Load();
		uint32 NewTail = Buffer.GetNextIndex(CurrentTail);

		if (NewTail != Head.Load())
		{
			Buffer[CurrentTail] = Element;
			Tail.Store(NewTail);

			return true;
		}

		return false;
	}

	/**
	 * Adds an item to the end of the queue.
	 *
	 * @param Element The element to add.
	 * @return true if the item was added, false if the queue was full.
	 * @note To be called only from producer thread.
	 */
	bool Enqueue(FElementType&& Element)
	{
		const uint32 CurrentTail = Tail.Load();
		uint32 NewTail = Buffer.GetNextIndex(CurrentTail);

		if (NewTail != Head.Load())
		{
			Buffer[CurrentTail] = MoveTemp(Element);
			Tail.Store(NewTail);

			return true;
		}

		return false;
	}

	/**
	 * Checks whether the queue is empty.
	 *
	 * Can be called from any thread. The result reflects the calling thread's current
	 * view. Since no locking is used, different threads may return different results.
	 *
	 * @return true if the queue is empty, false otherwise.
	 * @see Empty, IsFull
	 */
	UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return (Head.Load() == Tail.Load());
	}

	/**
	 * Checks whether the queue is full.
	 *
	 * Can be called from any thread. The result reflects the calling thread's current
	 * view. Since no locking is used, different threads may return different results.
	 *
	 * @return true if the queue is full, false otherwise.
	 * @see IsEmpty
	 */
	bool IsFull() const
	{
		return (Buffer.GetNextIndex(Tail.Load()) == Head.Load());
	}

	/**
	 * Returns the oldest item in the queue without removing it.
	 *
	 * @param OutItem Will contain the item if the queue is not empty.
	 * @return true if an item has been returned, false if the queue was empty.
	 * @note To be called only from consumer thread.
	 */
	bool Peek(FElementType& OutItem) const
	{
		const uint32 CurrentHead = Head.Load();

		if (CurrentHead != Tail.Load())
		{
			OutItem = Buffer[CurrentHead];

			return true;
		}

		return false;
	}

	/**
	 * Returns the oldest item in the queue without removing it.
	 *
	 * @return an FElementType pointer if an item has been returned, nullptr if the queue was empty.
	 * @note To be called only from consumer thread.
	 * @note The return value is only valid until Dequeue, Empty, or the destructor has been called.
	 */
	const FElementType* Peek() const
	{
		const uint32 CurrentHead = Head.Load();

		if (CurrentHead != Tail.Load())
		{
			return &Buffer[CurrentHead];
		}

		return nullptr;
	}

private:

	/** Holds the buffer. */
	TCircularBuffer<FElementType> Buffer;

	/** Holds the index to the first item in the buffer. */
	TAtomic<uint32> Head;

	/** Holds the index to the last item in the buffer. */
	TAtomic<uint32> Tail;
};
