// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "CoreTypes.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/UnrealTemplate.h"
#include <atomic>

/** 
 * Multi-producer/single-consumer unbounded concurrent queue that can be consumed only once.
 */
template<typename T>
class TClosableMpscQueue final
{
public:
	UE_NONCOPYABLE(TClosableMpscQueue);

	[[nodiscard]] TClosableMpscQueue() = default;

	~TClosableMpscQueue()
	{
		if (Head.load(std::memory_order_relaxed) == nullptr)
		{
			return; // closed
		}

		FNode* Tail = Sentinel.Next.load(std::memory_order_relaxed);
		while (Tail != nullptr)
		{
			FNode* Next = Tail->Next.load(std::memory_order_relaxed);
			DestructItem((T*)&Tail->Value);
			delete Tail;
			Tail = Next;
		}
	}

	/**
	 * Returns false if the queue is closed
	 */
	template <typename... ArgTypes>
	bool Enqueue(ArgTypes&&... Args)
	{
		FNode* Prev = Head.load(std::memory_order_acquire);
		if (Prev == nullptr)
		{
			return false; // already closed
		}

		FNode* New = new FNode;
		::new ((void*)&New->Value) T(Forward<ArgTypes>(Args)...);

		while (!Head.compare_exchange_weak(Prev, New, std::memory_order_release) && Prev != nullptr) // linearisation point
		{
		}

		if (Prev == nullptr)
		{
			DestructItem((T*)&New->Value);
			delete New;
			return false;
		}

		Prev->Next.store(New, std::memory_order_release);

		return true;
	}

	/**
	 * Closes the queue and consumes all items.
	 * @param Consumer: a functor with signature `AnyReturnType (T Value)` that will receive all items in FIFO order
	 * @return false if already closed
	 */
	template<typename F>
	bool Close(const F& Consumer)
	{
		FNode* Tail = &Sentinel;

		// we need to take note of the head at the moment of nullifying it because it can be still unreacheable from the tail
		FNode* const Head_Local = Head.exchange(nullptr, std::memory_order_acq_rel); // linearisation point

		// the queue is closed at this point, and the user is free to destroy it
		// no members should be accessed
		Close_NonMember(Head_Local, Tail, Consumer);

		return Head_Local != nullptr;
	}

	[[nodiscard]] bool IsClosed() const
	{
		return Head.load(std::memory_order_relaxed) == nullptr;
	}

private:
	struct FNode
	{
		std::atomic<FNode*> Next{ nullptr };
		TTypeCompatibleBytes<T> Value;
	};

	FNode Sentinel;
	std::atomic<FNode*> Head{ &Sentinel };

private:
	template<typename F>
	static void Close_NonMember(FNode* Head, FNode* Tail, const F& Consumer)
	{
		if (Head == Tail /* empty */ || Head == nullptr /* already closed */)
		{
			return;
		}

		auto GetNext = [](FNode* Node)
		{
			FNode* Next;
			// producers can be still updating `Next`, we need to loop until we detect that the list is fully linked
			do
			{
				Next = Node->Next.load(std::memory_order_relaxed);
			} while (Next == nullptr); // <- This loop has the potential for live locking if enqueue was not completed (e.g was running at lower priority)

			return Next;
		};

		// skip sentinel, do it outside of the main loop to avoid unnecessary branching in it
		Tail = GetNext(Tail);

		auto Consume = [&Consumer](FNode* Node)
		{
			T* ValuePtr = (T*)&Node->Value;
			Consumer(MoveTemp(*ValuePtr));
			DestructItem(ValuePtr);
			delete Node;
		};

		while (Tail != Head)
		{
			FNode* Next = GetNext(Tail);
			Consume(Tail);
			Tail = Next;
		}

		Consume(Head);
	}
};
