// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"

/* Compact hash table has two distinct features to keep it small
	1. The Index type adapts to the number of elements, so a smaller table will use a smaller type
	2. Holes in the table are patched up so all slots up to count are guaranteed to be valid

	For performance reasons we only reset the HashTable part of the lookup table since the indexes will be unused until added.
	It's the users responsibility to maintain correct item count
	!!Important!! It's also the users reponsibility to move items from the end of the list into their new spots when removing items.
	
	You are be able to maintain different views (hashes) of the same data since hole patching is deterministic between two tables.
	This can be used to allow searching against the same data using different fields as names
*/

/* Helper to write code against specific types resolves by number of elements. Usage:
	#define UE_COMPACTHASHTABLE_EXECUTEBYTYPE(Type) // Code
	UE_COMPACTHASHTABLE_CALLBYTYPE(NumElements)
	#undef UE_COMPACTHASHTABLE_EXECUTEBYTYPE
*/
#define UE_COMPACTHASHTABLE_CALLBYTYPE(NextIndexCount) \
	switch (UE::Core::CompactHashTable::GetTypeSize(NextIndexCount)) { \
	case 1: { UE_COMPACTHASHTABLE_EXECUTEBYTYPE(uint8); } break; \
	case 2: { UE_COMPACTHASHTABLE_EXECUTEBYTYPE(uint16); } break; \
	case 4: { UE_COMPACTHASHTABLE_EXECUTEBYTYPE(uint32); } break; \
	default: checkNoEntry(); }

namespace UE::Core::CompactHashTable::Private
{
	// The reference to the given index in chain of a given hash key and return a pointer to it
	template<typename IndexType>
	inline IndexType* FindLookupEntry(const uint32 Index, const uint32 Key, IndexType* HashData, const uint32 HashCount, IndexType* NextIndexData, const uint32 NextIndexCount, const TCHAR* Context)
	{
		const uint32 HashIndex = Key & (HashCount - 1);

		for (IndexType* Lookup = &HashData[HashIndex]; *Lookup < NextIndexCount; Lookup = &NextIndexData[*Lookup])
		{
			if (*Lookup == Index)
			{
				return Lookup;
			}
		}

		checkf(false, TEXT("Failed to find lookup entry from set from: %s."), Context); // This usually indicates that something changed in the entry that affected the hash so it no longer matches up.
		return nullptr; // will crash
	}
}

/* Helpers for interacting with compact hash table memory layouts */
namespace UE::Core::CompactHashTable
{
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr uint32 GetTypeSize(uint32 IndexCount)
	{
		return 1 + int(IndexCount > 0xff) + int(IndexCount > 0xffff) * 2;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT constexpr uint32 GetTypeShift(uint32 IndexCount)
	{
		return 0 + int(IndexCount > 0xff) + int(IndexCount > 0xffff);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT constexpr size_t GetMemoryRequiredInBytes(uint32 IndexCount, uint32 HashCount)
	{
		return (((size_t)IndexCount * UE::Core::Private::CompactSetAllocatorHelpers::KeyStorageMultiplier) + HashCount) << GetTypeShift(IndexCount);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT constexpr size_t GetMemoryAlignment()
	{
		return 4; // only support uint32 for now, pick highest alignment so we don't change alignment between allocations
	}

	template<typename IndexType>
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr uint32 IndexTypeToSizeType(const IndexType Index)
	{
		if constexpr (sizeof(IndexType) == 4)
		{
			return Index;
		}
		else
		{
			// Convert smaller type INDEX_NONE to uint32 INDEX_NONE if data is invalid
			constexpr IndexType InvalidIndex = (IndexType)INDEX_NONE;
			return Index == InvalidIndex ? (uint32)INDEX_NONE : (uint32)Index;
		}
	}

	/* Calculate the size of the hash table from the number of elements in the set */
	[[nodiscard]] inline constexpr uint32 GetHashCount(uint32 NumElements)
	{
		if (!NumElements)
		{
			return 0;
		}
		if (NumElements < 8)
		{
			return 4;
		}

		// Always use the power of 2 smaller than the current size to prioritize size over speed just a little bit
		return FGenericPlatformMath::RoundUpToPowerOfTwo(NumElements / 2 + 1); // 255 == 128, 256 == 256
	}

	/* Return the first index of a key from the hash table portion */
	template<typename IndexType>
	[[nodiscard]] inline uint32 GetFirst(uint32 Key, const IndexType* HashData, const uint32 HashCount)
	{
		const uint32 HashIndex = Key & (HashCount - 1);
		return IndexTypeToSizeType(HashData[HashIndex]);
	}

	/* Return the first index of a key from the hash table portion */
	template<typename IndexType>
	[[nodiscard]] inline uint32 GetFirstByIndex(uint32 HashIndex, const IndexType* HashData, const uint32 HashCount)
	{
		checkSlow(HashIndex < HashCount);
		return IndexTypeToSizeType(HashData[HashIndex]);
	}

	/* Given an existing index, return the next index in case there was a collision in the has table */
	template<typename IndexType>
	[[nodiscard]] inline uint32 GetNext(uint32 Index, const IndexType* NextIndexData, const uint32 NextIndexCount)
	{
		checkSlow(Index < NextIndexCount);
		return IndexTypeToSizeType(NextIndexData[Index]);
	}

	/* Do a full search for an existing element in the table given a function to compare if a found element is what you're looking for */
	template<typename IndexType, typename PredicateType>
	[[nodiscard]] inline uint32 Find(uint32 Key, const IndexType* HashData, const uint32 HashCount, const IndexType* NextIndexData, const uint32 NextIndexCount, const PredicateType& Predicate)
	{
		for (IndexType ElementIndex = HashData[Key & (HashCount - 1)]; ElementIndex != (IndexType)INDEX_NONE; ElementIndex = NextIndexData[ElementIndex])
		{
			if (Predicate(ElementIndex))
			{
				// Return the first match, regardless of whether the set has multiple matches for the key or not.
				return ElementIndex;
			}
			checkSlow(ElementIndex < NextIndexCount);
		}
		return INDEX_NONE;
	}
	
	/* Insert new element into the hash table */
	template<typename IndexType>
	inline void Add(uint32 Index, uint32 Key, IndexType* HashData, const uint32 HashCount, IndexType* NextIndexData, const uint32 NextIndexCount)
	{
		checkSlow(Index < NextIndexCount);

		const uint32 HashIndex = Key & (HashCount - 1);
		NextIndexData[Index] = HashData[HashIndex];
		HashData[HashIndex] = (IndexType)Index;
	}

	/* Remove an element from the list, move the last element into the now empty slot
	 If the item to remove is the last element then the last element's key will be ignored (you can skip calculating it if it's expensive)
	*/
	template<typename IndexType>
	inline uint32 Remove(const uint32 Index, const uint32 Key, const uint32 LastIndex, uint32 OptLastKey, IndexType* HashData, const uint32 HashCount, IndexType* NextIndexData, const uint32 NextIndexCount)
	{
		checkSlow(LastIndex < NextIndexCount && Index <= LastIndex);

		// Next = Next->Next
		IndexType* LookupEntry = UE::Core::CompactHashTable::Private::FindLookupEntry<IndexType>(Index, Key, HashData, HashCount, NextIndexData, NextIndexCount, TEXT("Remove initial lookup"));
		*LookupEntry = NextIndexData[Index];

		if (Index != LastIndex)
		{
			// Remap the last element into the now empty spot
			IndexType* LastLookupEntry = UE::Core::CompactHashTable::Private::FindLookupEntry<IndexType>(LastIndex, OptLastKey, HashData, HashCount, NextIndexData, NextIndexCount, TEXT("Remove last element lookup"));
			*LastLookupEntry = (IndexType)Index;

			// Move the last elements next index into the new slot as well
			NextIndexData[Index] = NextIndexData[LastIndex];
		}

		// Return whatever next is at the original lookup entry that we fixed up. If it happened to be the last index the the previous block would have fixed it up for us
		return IndexTypeToSizeType(*LookupEntry);
	}

	/* Remove an element from the list, shift all indexes down to preserve the order of elements in a list
	 This is a very expensive operation so it should only be used if absolutely necessary (i.e. generally only for user facing data)
	*/
	template<typename IndexType>
	inline uint32 RemoveStable(const uint32 Index, const uint32 Key, IndexType* HashData, const uint32 HashCount, IndexType* NextIndexData, const uint32 NextIndexCount)
	{
		checkSlow(Index < NextIndexCount);

		constexpr IndexType InvalidIndex = (IndexType)INDEX_NONE;

		// If TestIndex is valid and above Index just remove 1 so it will reference the new location that elements should get relocated to
		auto FixIndex = [Index](IndexType& TestIndex)
		{
			if (TestIndex != InvalidIndex && TestIndex > Index)
			{
				--TestIndex;
			}
		};

		IndexType* LookupEntry = UE::Core::CompactHashTable::Private::FindLookupEntry<IndexType>(Index, Key, HashData, HashCount, NextIndexData, NextIndexCount, TEXT("Remove stable"));
		*LookupEntry = NextIndexData[Index];
		FixIndex(*LookupEntry);

		// Cache return value here since the memory location could move if it was a 'next index' that's located after where Index is
		const uint32 NextIndex = IndexTypeToSizeType(*LookupEntry);

		// For the hash indexes, just decrement any that are bigger than the removed element
		for (uint32 HashIndex = 0; HashIndex < HashCount; ++HashIndex)
		{
			FixIndex(HashData[HashIndex]);
		}

		// Decrement values for all next index elements that are before the removed element
		for (uint32 NextIndexIndex = 0; NextIndexIndex < Index; ++NextIndexIndex)
		{
			FixIndex(NextIndexData[NextIndexIndex]);
		}

		// Move AND Decrement values for all next index elements that are after the removed element
		for (uint32 NextIndexIndex = Index + 1; NextIndexIndex < NextIndexCount; ++NextIndexIndex)
		{
			// Not using FixIndex so we can use the else
			if (NextIndexData[NextIndexIndex] != InvalidIndex && NextIndexData[NextIndexIndex] > Index)
			{
				NextIndexData[NextIndexIndex - 1] = NextIndexData[NextIndexIndex] - 1;
			}
			else
			{
				NextIndexData[NextIndexIndex - 1] = NextIndexData[NextIndexIndex];
			}
		}

		return NextIndex;
	}

	/*
	 * Only remove the hash element, not the next index, this is only to be used when clearing the hash data to only hit the specific hash elements instead of writing the entirety of memory
	 */
	template<typename IndexType>
	inline void RemoveHashElement(const uint32 Key, IndexType* HashData, const uint32 HashCount)
	{
		const uint32 HashIndex = Key & (HashCount - 1);
		HashData[HashIndex] = (IndexType)INDEX_NONE;
	}
}

/* Helper to lookup a type from a type size, todo: Add support for 3 byte lookup */
template<uint32 TypeSize> struct TCompactHashTypeLookupBySize;
template<> struct TCompactHashTypeLookupBySize<1> { using Type = uint8; };
template<> struct TCompactHashTypeLookupBySize<2> { using Type = uint16; };
template<> struct TCompactHashTypeLookupBySize<4> { using Type = uint32; };

/* Helper for making a fixed size hash table that manages its own memory */
template<uint32 ElementCount, uint32 HashCount = UE::Core::CompactHashTable::GetHashCount(ElementCount)>
class TStaticCompactHashTable
{
public:
	using IndexType = typename TCompactHashTypeLookupBySize<UE::Core::CompactHashTable::GetTypeSize(ElementCount)>::Type;

	TStaticCompactHashTable()
	{
		static_assert(FMath::IsPowerOfTwo(HashCount));
		Reset();
	}
	TStaticCompactHashTable(ENoInit) {}
	
	UE_FORCEINLINE_HINT void Reset()
	{
		FMemory::Memset(HashData, 0xff, sizeof(HashData));
	}

	[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetFirst(uint32 Key) const
	{
		return UE::Core::CompactHashTable::GetFirst(Key, HashData, HashCount);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetFirstByIndex(uint32 HashIndex) const
	{
		return UE::Core::CompactHashTable::GetFirstByIndex(HashIndex, HashData, HashCount);
	}

	[[nodiscard]] inline uint32 GetNext(uint32 Index, uint32 CurrentCount) const
	{
		checkSlow(CurrentCount <= ElementCount);
		return UE::Core::CompactHashTable::GetNext(Index, NextIndexData, CurrentCount);
	}

	template<typename PredicateType>
	[[nodiscard]] inline uint32 Find(uint32 Key, uint32 CurrentCount, const PredicateType& Predicate) const
	{
		checkSlow(CurrentCount <= ElementCount);
		return UE::Core::CompactHashTable::Find(Key, HashData, HashCount, NextIndexData, CurrentCount, Predicate);
	}
	
	inline void Add(uint32 CurrentCount, uint32 Key)
	{
		checkSlow(CurrentCount < ElementCount);
		UE::Core::CompactHashTable::Add(CurrentCount, Key, HashData, HashCount, NextIndexData, CurrentCount + 1);
	}

	UE_FORCEINLINE_HINT void Remove(uint32 Index, uint32 Key, uint32 LastIndex, uint32 OptLastKey)
	{
		UE::Core::CompactHashTable::Remove(Index, Key, LastIndex, OptLastKey, HashData, HashCount, NextIndexData, ElementCount);
	}

protected:
	IndexType NextIndexData[ElementCount]; // Collision redirector to next index for keys that hash to the same initial index
	IndexType HashData[HashCount]; // First index lookup from key
};

/* Helper for interacting with existing hash table memory */
class FConstCompactHashTableView
{
public:
	static constexpr uint8 KeyStorageMultiplier = UE::Core::Private::CompactSetAllocatorHelpers::KeyStorageMultiplier;

	explicit FConstCompactHashTableView() = default;

	inline explicit FConstCompactHashTableView(const uint8* Memory, uint32 InNextIndexCount, uint32 InHashCount, size_t MemorySize)
	: NextIndexData(Memory)
	#if UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE
	, KeyStorage(Memory + (size_t(InNextIndexCount) << UE::Core::CompactHashTable::GetTypeShift(InNextIndexCount)))
	#endif
	, HashData(Memory + (size_t(InNextIndexCount) << UE::Core::CompactHashTable::GetTypeShift(InNextIndexCount)) * KeyStorageMultiplier)
	, NextIndexCount(InNextIndexCount)
	, HashCount(InHashCount)
	{
		checkSlow(Memory != nullptr && InNextIndexCount > 0 && InHashCount > 0 && MemorySize > 0);
		checkfSlow(MemorySize == UE::Core::CompactHashTable::GetMemoryRequiredInBytes(NextIndexCount, HashCount), TEXT("Got %zu bytes, expected %zu bytes"), MemorySize, UE::Core::CompactHashTable::GetMemoryRequiredInBytes(NextIndexCount, HashCount));
		checkSlow(FMath::IsPowerOfTwo(HashCount));
	}

	[[nodiscard]] uint32 GetHashCount() const
	{
		return HashCount;
	}

	// Functions used to search
	[[nodiscard]] inline uint32 GetFirst(uint32 Key) const
	{
		#define UE_COMPACTHASHTABLE_EXECUTEBYTYPE(Type) return UE::Core::CompactHashTable::GetFirst(Key, (const Type*)HashData, HashCount)
		UE_COMPACTHASHTABLE_CALLBYTYPE(NextIndexCount);
		#undef UE_COMPACTHASHTABLE_EXECUTEBYTYPE
		return INDEX_NONE;
	}

	// Advanced used for manual inspection of the hash data
	[[nodiscard]] inline uint32 GetFirstByIndex(uint32 HashIndex) const
	{
		#define UE_COMPACTHASHTABLE_EXECUTEBYTYPE(Type) return UE::Core::CompactHashTable::GetFirstByIndex(HashIndex, (const Type*)HashData, HashCount)
		UE_COMPACTHASHTABLE_CALLBYTYPE(NextIndexCount);
		#undef UE_COMPACTHASHTABLE_EXECUTEBYTYPE
		return INDEX_NONE;
	}

	[[nodiscard]] inline uint32 GetNext(uint32 Index, uint32 CurrentCount) const
	{
		checkSlow(CurrentCount <= NextIndexCount);
	
		#define UE_COMPACTHASHTABLE_EXECUTEBYTYPE(Type) return UE::Core::CompactHashTable::GetNext(Index, (const Type*)NextIndexData, CurrentCount)
		UE_COMPACTHASHTABLE_CALLBYTYPE(NextIndexCount);
		#undef UE_COMPACTHASHTABLE_EXECUTEBYTYPE
		return INDEX_NONE;
	}

	template<typename PredicateType>
	[[nodiscard]] inline uint32 Find(uint32 Key, uint32 CurrentCount, const PredicateType& Predicate) const
	{
		checkSlow(CurrentCount <= NextIndexCount);
		#define UE_COMPACTHASHTABLE_EXECUTEBYTYPE(Type) return UE::Core::CompactHashTable::Find(Key, (const Type*)HashData, HashCount, (const Type*)NextIndexData, CurrentCount, Predicate)
		UE_COMPACTHASHTABLE_CALLBYTYPE(NextIndexCount);
		#undef UE_COMPACTHASHTABLE_EXECUTEBYTYPE
		return INDEX_NONE;
	}

	#if UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE
	// Functions used to search
	[[nodiscard]] inline uint32 GetStoredKey(uint32 Index, uint32 CurrentCount) const
	{
		checkSlow(CurrentCount <= NextIndexCount);
	
		#define UE_COMPACTHASHTABLE_EXECUTEBYTYPE(Type) return ((const Type*)KeyStorage)[Index];
		UE_COMPACTHASHTABLE_CALLBYTYPE(NextIndexCount);
		#undef UE_COMPACTHASHTABLE_EXECUTEBYTYPE
		return INDEX_NONE;
	}
	#endif

protected:
	const uint8* NextIndexData = nullptr;

	#if UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE
	const uint8* KeyStorage = nullptr;
	#endif

	const uint8* HashData = nullptr;
	uint32 NextIndexCount = 0;
	uint32 HashCount = 0;
};

/* Helper for interacting with existing hash table memory */
class FCompactHashTableView : public FConstCompactHashTableView
{
public:
	UE_FORCEINLINE_HINT explicit FCompactHashTableView(uint8* Memory, uint32 InNextIndexCount, uint32 InHashCount, size_t MemorySize)
	: FConstCompactHashTableView(Memory, InNextIndexCount, InHashCount, MemorySize)
	{
	}

	inline void Reset() const
	{
		const uint32 TypeShift = UE::Core::CompactHashTable::GetTypeShift(NextIndexCount);

		// The const_casts are a little gross but I don't wan't to maintain duplicate copies of the const functions
		FMemory::Memset(const_cast<uint8*>(HashData), 0xff, size_t(HashCount) << TypeShift);
	}
	
	inline void Add(uint32 Index, uint32 Key) const
	{
		checkSlow(Index < NextIndexCount);
		#if UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE
		#define UE_COMPACTHASHTABLE_EXECUTEBYTYPE(Type) UE::Core::CompactHashTable::Add(Index, Key, (Type*)(HashData), HashCount, (Type*)(NextIndexData), NextIndexCount); \
			((Type*)KeyStorage)[Index] = (Type)(Key & (HashCount - 1))
		#else
		#define UE_COMPACTHASHTABLE_EXECUTEBYTYPE(Type) UE::Core::CompactHashTable::Add(Index, Key, (Type*)(HashData), HashCount, (Type*)(NextIndexData), NextIndexCount)
		#endif
		UE_COMPACTHASHTABLE_CALLBYTYPE(NextIndexCount);
		#undef UE_COMPACTHASHTABLE_EXECUTEBYTYPE
	}

	UE_FORCEINLINE_HINT uint32 Remove(uint32 Index, uint32 Key, uint32 LastIndex, uint32 OptLastKey) const
	{
		#if UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE
		#define UE_COMPACTHASHTABLE_EXECUTEBYTYPE(Type) ((Type*)KeyStorage)[Index] = ((Type*)KeyStorage)[LastIndex]; \
			return UE::Core::CompactHashTable::Remove(Index, Key, LastIndex, OptLastKey, (Type*)(HashData), HashCount, (Type*)(NextIndexData), NextIndexCount)
		#else
		#define UE_COMPACTHASHTABLE_EXECUTEBYTYPE(Type) return UE::Core::CompactHashTable::Remove(Index, Key, LastIndex, OptLastKey, (Type*)(HashData), HashCount, (Type*)(NextIndexData), NextIndexCount)
		#endif
		UE_COMPACTHASHTABLE_CALLBYTYPE(NextIndexCount);
		#undef UE_COMPACTHASHTABLE_EXECUTEBYTYPE
		return INDEX_NONE;
	}

	UE_FORCEINLINE_HINT uint32 RemoveStable(uint32 Index, uint32 Key) const
	{
		#if UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE
		#define UE_COMPACTHASHTABLE_EXECUTEBYTYPE(Type) FMemory::Memmove(((Type*)KeyStorage) + Index, ((Type*)KeyStorage) + Index + 1, (NextIndexCount - Index - 1) * sizeof(Type)); \
			return UE::Core::CompactHashTable::RemoveStable(Index, Key, (Type*)(HashData), HashCount, (Type*)(NextIndexData), NextIndexCount)
		#else
		#define UE_COMPACTHASHTABLE_EXECUTEBYTYPE(Type) return UE::Core::CompactHashTable::RemoveStable(Index, Key, (Type*)(HashData), HashCount, (Type*)(NextIndexData), NextIndexCount)
		#endif
		UE_COMPACTHASHTABLE_CALLBYTYPE(NextIndexCount);
		#undef UE_COMPACTHASHTABLE_EXECUTEBYTYPE
		return INDEX_NONE;
	}

	UE_FORCEINLINE_HINT void RemoveHashElement(uint32 Key) const
	{
		#define UE_COMPACTHASHTABLE_EXECUTEBYTYPE(Type) UE::Core::CompactHashTable::RemoveHashElement(Key, (Type*)(HashData), HashCount)
		UE_COMPACTHASHTABLE_CALLBYTYPE(NextIndexCount);
		#undef UE_COMPACTHASHTABLE_EXECUTEBYTYPE
	}
};
