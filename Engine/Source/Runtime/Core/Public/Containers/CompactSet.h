// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define UE_TCOMPACT_SET_FORCE_ARRAY_VIEW // always enabled for explicitly typed compact sets
#define UE_TCOMPACT_SET TCompactSet
#include "CompactSet.h.inl"
#undef UE_TCOMPACT_SET
#undef UE_TCOMPACT_SET_FORCE_ARRAY_VIEW