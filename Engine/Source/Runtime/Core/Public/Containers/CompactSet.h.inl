// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef UE_TCOMPACT_SET
#error "CompactSet.h.inl should only be included after defining UE_TCOMPACT_SET"
#endif

#include "ContainersFwd.h"
#include "Containers/CompactSetBase.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/ContainerElementTypeCompatibility.h"
#include "Containers/SetUtilities.h"
#include "Containers/UnrealString.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/MemoryImageWriter.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/RetainedRef.h"
#include "Templates/TypeHash.h"
#include "Traits/IsTriviallyRelocatable.h"

#include <initializer_list>
#include <type_traits>

namespace UE::Core::Private
{
	[[noreturn]] CORE_API void OnInvalidSetNum(unsigned long long NewNum);
}

// Array view is a unique feature for compact sets since the data is not allowed to have holes like sparse set
// However to maintain compatibilty with sparse set this feature is not available by default
#if !defined(UE_TCOMPACT_SET_ARRAY_VIEW)
#define UE_TCOMPACT_SET_ARRAY_VIEW 0
#endif

#if UE_TCOMPACT_SET_ARRAY_VIEW || defined(UE_TCOMPACT_SET_FORCE_ARRAY_VIEW)
#define UE_TCOMPACT_SET_ARRAY_VIEW_ENABLED 1
#else
#define UE_TCOMPACT_SET_ARRAY_VIEW_ENABLED 0
#endif

#define TSETPRIVATEFRIEND UE_JOIN(UE_TCOMPACT_SET, PrivateFriend)

/**
 * A set with an optional KeyFuncs parameters for customizing how the elements are compared and searched.  
 * E.g. You can specify a mapping from elements to keys if you want to find elements by specifying a subset of 
 * the element type.  It uses a TSparseArray of the elements, and also links the elements into a hash with a 
 * number of buckets proportional to the number of elements.  Addition, removal, and finding are O(1).
 *
 * The ByHash() functions are somewhat dangerous but particularly useful in two scenarios:
 * -- Heterogeneous lookup to avoid creating expensive keys like FString when looking up by const TCHAR*.
 *	  You must ensure the hash is calculated in the same way as ElementType is hashed.
 *    If possible put both ComparableKey and ElementType hash functions next to each other in the same header
 *    to avoid bugs when the ElementType hash function is changed.
 * -- Reducing contention around hash tables protected by a lock. It is often important to incur
 *    the cache misses of reading key data and doing the hashing *before* acquiring the lock.
 *
 * Here's a visual example of the data layout for the compact set
 *  ____________ ___________ ________________ _____________ ____________
 * |            |           |                | (Optional)  |            |
 * | Data Array | Hash Size | Collision List | Key Storage | Hash Table |
 * |____________|___________|________________|_____________|____________|
 * 
 * Data Array - Payload of the set. This is a just a regular array of items without any empty spots
 * Hash Size - 4 byte integer to reference how big the hash table is. Storing this is significantly faster than trying to recalculate it
 * Collision List - For each entry in the Data Array there is an entry in this list that may contain a valid index to the next item to consider for hash table collisions
 * (Optional) Key Storage - If enabled, stores "GetHashKey(Item) % (HashSize - 1)" so we can remove an item without having to rehash it.
 * Hash Table - Power of 2 table to lookup the first index of a given hash value
 *
 **/
template<typename InElementType, typename KeyFuncs /*= DefaultKeyFuncs<ElementType>*/, typename Allocator /*= FDefaultSetAllocator*/>
class alignas(Allocator::template AllocatorAlignment<InElementType>::Value) UE_TCOMPACT_SET : public TCompactSetBase<typename Allocator::template ElementAllocator<sizeof(InElementType)>>
{
public:
	using ElementType = InElementType;
	using KeyFuncsType = KeyFuncs;

private:
	using Super = TCompactSetBase<typename Allocator::template ElementAllocator<sizeof(InElementType)>>;
	using typename Super::SizeType;
	using typename Super::HashCountType;
	using typename Super::AllocatorType;
	using USizeType = std::make_unsigned_t<SizeType>;
	using KeyInitType = typename KeyFuncs::KeyInitType;
	using ElementInitType = typename KeyFuncs::ElementInitType;

	// Required so TScriptCompactSet can validate it's layout matches this type
	template <typename ScriptAllocator>
	friend class TScriptCompactSet;

	// Private helper functions to help binding to global functions
	friend struct TSETPRIVATEFRIEND;

	/** The base type of whole set iterators. */
	template<bool bConst>
	class TBaseIterator
	{
	private:
		using SetType = std::conditional_t<bConst, const UE_TCOMPACT_SET, UE_TCOMPACT_SET>;

	public:
		using ElementItType = std::conditional_t<bConst, const ElementType, ElementType>;

		[[nodiscard]] explicit UE_FORCEINLINE_HINT TBaseIterator(SetType& InSet)
		: TBaseIterator(InSet, 0)
		{
		}

		[[nodiscard]] explicit inline TBaseIterator(SetType& InSet, SizeType StartIndex)
		: Set(InSet)
		, Index(StartIndex)
#if DO_CHECK
		, InitialNum(InSet.Num())
#endif
		{
#if DO_CHECK
			check(StartIndex >= 0 && StartIndex <= InitialNum);
#endif
		}

		[[nodiscard]] UE_FORCEINLINE_HINT ElementItType & operator*() const
		{
#if DO_CHECK
			checkf(LastRemoved == INDEX_NONE, TEXT("Accessing iterator after it was removed from the set and before it was incremented to the next valid elements."));
#endif
			return Set[FSetElementId::FromInteger(Index)];
		}

		[[nodiscard]] UE_FORCEINLINE_HINT ElementItType* operator->() const
		{
#if DO_CHECK
			checkf(LastRemoved == INDEX_NONE, TEXT("Accessing iterator after it was removed from the set and before it was incremented to the next valid elements."));
#endif
			return &Set[FSetElementId::FromInteger(Index)];
		}

		UE_FORCEINLINE_HINT TBaseIterator& operator++()
		{
#if DO_CHECK
			const int32 SetNum = Set.Num();
			checkf(SetNum >= InitialNum, TEXT("Sets/Maps should never have elements removed during iteration outside of Iterator.RemoveCurrent(). InitialNum %d CurrentNum %d"), InitialNum, SetNum);
			LastRemoved = INDEX_NONE;
#endif
			++Index;
			return *this;
		}

		/** conversion to "bool" returning true if the iterator is valid. */
		[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
		{
			return Set.IsValidId(this->GetId());
		}

		[[nodiscard]] inline bool operator==(const TBaseIterator& Rhs) const
		{
			checkSlow(&Set == &Rhs.Set);
			return Index == Rhs.Index;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TBaseIterator& Rhs) const
		{
			return !(*this == Rhs);
		}

		// Accessors.
		[[nodiscard]] UE_FORCEINLINE_HINT FSetElementId GetId() const
		{
			return FSetElementId::FromInteger(Index);
		}
		
	protected:
		SetType& Set;
		SizeType Index;
#if DO_CHECK
		SizeType InitialNum;
		SizeType LastRemoved = INDEX_NONE;
#endif
	};

	/** The base type of whole set iterators. */
	template<bool bConst>
	class TBaseKeyIterator
	{
	private:
		typedef std::conditional_t < bConst, const UE_TCOMPACT_SET, UE_TCOMPACT_SET > SetType;
		typedef std::conditional_t < bConst, const ElementType, ElementType > ItElementType;
		typedef typename TTypeTraits<typename KeyFuncs::KeyType>::ConstPointerType ReferenceOrValueType;

	public:
		using KeyArgumentType = std::conditional_t <
			std::is_reference_v<ReferenceOrValueType> ,
			TRetainedRef<std::remove_reference_t<ReferenceOrValueType>> ,
			KeyInitType
		>;

		/** Initialization constructor. */
		[[nodiscard]] explicit inline TBaseKeyIterator(SetType& InSet, KeyArgumentType InKey)
		: Set  (InSet)
		, Key  (InKey)
		, HashTable(InSet.Num() ? Set.GetConstHashTableView() : FConstCompactHashTableView())
		, NextIndex(InSet.Num() ? HashTable.GetFirst(KeyFuncs::GetKeyHash(Key)) : INDEX_NONE)
#if DO_CHECK
		, InitialNum(InSet.Num())
#endif
		{
			++(*this);
		}

		/** Advances the iterator to the next element. */
		inline TBaseKeyIterator& operator++()
		{
			const int32 SetNum = Set.Num();

			// Note: Adding new elements is safe, however they will be guaranteed to be missed by the current iteration
#if DO_CHECK
			checkf(SetNum >= InitialNum, TEXT("Sets/Maps should never have elements removed during iteration outside of Iterator.RemoveCurrent(). InitialNum %d CurrentNum %d"), InitialNum, SetNum);
			LastRemoved = INDEX_NONE;
#endif

			Index = NextIndex;

			while (Index != INDEX_NONE)
			{
				NextIndex = HashTable.GetNext(Index, SetNum);
				checkSlow(Index != NextIndex);

				if (KeyFuncs::Matches(KeyFuncs::GetSetKey(Set[GetId()]), Key))
				{
					break;
				}

				Index = NextIndex;
			}

			return *this;
		}

		/** conversion to "bool" returning true if the iterator is valid. */
		[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
		{
			return Index != INDEX_NONE;
		}

		// Accessors.
		[[nodiscard]] UE_FORCEINLINE_HINT FSetElementId GetId() const
		{
			return FSetElementId::FromInteger(this->Index);
		}

		[[nodiscard]] UE_FORCEINLINE_HINT ItElementType* operator->() const
		{
			return &Set[GetId()];
		}

		[[nodiscard]] UE_FORCEINLINE_HINT ItElementType& operator*() const
		{
			return Set[GetId()];
		}

	protected:
		SetType& Set;
		ReferenceOrValueType Key;
		FConstCompactHashTableView HashTable;
		SizeType Index;
		SizeType NextIndex;

#if DO_CHECK
		SizeType InitialNum;
		SizeType LastRemoved = INDEX_NONE;
#endif
	};

public:

	/** Used to iterate over the elements of a const UE_TCOMPACT_SET. */
	using TConstIterator = TBaseIterator<true>;

	/** Used to iterate over the elements of a UE_TCOMPACT_SET. */
	class TIterator : public TBaseIterator<false>
	{
	private:
		using Super = TBaseIterator<false>;

	public:
		using Super::Super;

		/** Removes the current element from the set. */
		inline void RemoveCurrent()
		{
#if DO_CHECK
			checkf(this->LastRemoved == INDEX_NONE, TEXT("RemoveCurrent was already called for the current iteration index of %d"), this->LastRemoved);
			this->LastRemoved = this->Index;
			--this->InitialNum;
#endif

			this->Set.Remove(this->GetId());
			--this->Index;
		}
	};

	/** Used to iterate over the elements of a const UE_TCOMPACT_SET. */
	using TConstKeyIterator = TBaseKeyIterator<true>;

	/** Used to iterate over the elements of a UE_TCOMPACT_SET. */
	class TKeyIterator : public TBaseKeyIterator<false>
	{
	private:
		using Super = TBaseKeyIterator<false>;

	public:
		using KeyArgumentType = typename Super::KeyArgumentType;
		using Super::Super;

		/** Removes the current element from the set. */
		inline void RemoveCurrent()
		{
#if DO_CHECK
			checkf(this->LastRemoved == INDEX_NONE, TEXT("RemoveCurrent was already called for the current iteration index of %d"), this->LastRemoved);
			this->LastRemoved = this->Index;
			--this->InitialNum;
#endif

			this->Set.Remove(this->GetId());

			// If the next element was the last in the set then it will get remapped to the current index, so fix that up
			if (this->NextIndex == this->Set.Num())
			{
				this->NextIndex = this->Index;
			}

			this->Index = INDEX_NONE;
		}
	};

	/* Constructors */

	[[nodiscard]] UE_FORCEINLINE_HINT constexpr UE_TCOMPACT_SET() = default;

	[[nodiscard]] explicit consteval UE_TCOMPACT_SET(EConstEval)
		: Super(ConstEval)
	{
	}

	[[nodiscard]] UE_FORCEINLINE_HINT UE_TCOMPACT_SET(const UE_TCOMPACT_SET& Copy)
	{
		*this = Copy;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT explicit UE_TCOMPACT_SET(TArrayView<const ElementType> InArrayView)
	{
		Append(InArrayView);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT explicit UE_TCOMPACT_SET(TArray<ElementType>&& InArray)
	{
		Append(MoveTemp(InArray));
	}

	[[nodiscard]] UE_TCOMPACT_SET(std::initializer_list<ElementType> InitList)
	{
		Append(InitList);
	}

	[[nodiscard]] UE_TCOMPACT_SET(UE_TCOMPACT_SET&& Other)
	{
		*this = MoveTemp(Other);
	}

	/** Constructor for moving elements from a UE_TCOMPACT_SET with a different SetAllocator */
	template<typename OtherAllocator>
	[[nodiscard]] UE_TCOMPACT_SET(UE_TCOMPACT_SET<ElementType, KeyFuncs, OtherAllocator>&& Other)
	{
		Append(MoveTemp(Other));
	}

	/** Constructor for copying elements from a UE_TCOMPACT_SET with a different SetAllocator */
	template<typename OtherAllocator>
	[[nodiscard]] UE_TCOMPACT_SET(const UE_TCOMPACT_SET<ElementType, KeyFuncs, OtherAllocator>& Other)
	{
		Append(Other);
	}

	UE_FORCEINLINE_HINT ~UE_TCOMPACT_SET()
	{
		UE_STATIC_ASSERT_WARN(TIsTriviallyRelocatable_V<InElementType>, "TMapBase can only be used with trivially relocatable types");

		Empty(0);
	}

	////////////////////////////////////////////////////
	// Start - intrusive TOptional<UE_TCOMPACT_SET> state //
	////////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = UE_TCOMPACT_SET;

	[[nodiscard]] explicit UE_TCOMPACT_SET(FIntrusiveUnsetOptionalState Tag) : Super(Tag)
	{
	}
	//////////////////////////////////////////////////
	// End - intrusive TOptional<UE_TCOMPACT_SET> state //
	//////////////////////////////////////////////////

	/* Assignment operators */

	UE_TCOMPACT_SET& operator=(const UE_TCOMPACT_SET& Copy)
	{
		if (this != &Copy)
		{
			// This could either take the full memory size of the Copy and prevent a rehash or
			// only allocate the required set but have to hash everything again (i.e. perf vs memory)
			// Could try a middle ground of allowing extra memory if it's within slack margins.
			// Going for memory savings for now

			// Not using Empty(NumElements) to avoid clearing the hash memory since we'll rebuild it anyway
			// We need to make sure the relevant parts are cleared so ResizeAllocation can run safely and with minimal cost
			DestructItems(GetData(), this->NumElements);
			this->NumElements = 0;

			ResizeAllocation(Copy.NumElements);

			this->NumElements = Copy.NumElements;
			ConstructItems<ElementType>(GetData(), Copy.GetData(), this->NumElements);

			Rehash();
		}
		return *this;
	}

	UE_TCOMPACT_SET& operator=(UE_TCOMPACT_SET&& Other)
	{
		if (this != &Other)
		{
			DestructItems(GetData(), this->NumElements);
			this->Elements.MoveToEmpty(Other.Elements);
			this->NumElements = Other.NumElements;
			this->MaxElements = Other.MaxElements;

			Other.NumElements = 0;
			Other.MaxElements = 0;
		}

		return *this;
	}

	/** Assignment operator for moving elements from a UE_TCOMPACT_SET with a different SetAllocator */
	template<typename OtherAllocator>
	UE_TCOMPACT_SET& operator=(UE_TCOMPACT_SET<ElementType, KeyFuncs, OtherAllocator> && Other)
	{
		Reset();
		Append(MoveTemp(Other));
		return *this;
	}

	/** Assignment operator for copying elements from a UE_TCOMPACT_SET with a different SetAllocator */
	template<typename OtherAllocator>
	UE_TCOMPACT_SET& operator=(const UE_TCOMPACT_SET<ElementType, KeyFuncs, OtherAllocator>& Other)
	{
		Reset();
		Append(Other);
		return *this;
	}

	UE_TCOMPACT_SET& operator=(std::initializer_list<ElementType> InitList)
	{
		Reset();
		Append(InitList);
		return *this;
	}

	/**
	 * Move assignment operator.
	 * Compatible element type version.
	 *
	 * @param Other Set to assign and move from.
	 */
	template <
		typename OtherKeyFuncs,
		typename AliasElementType = ElementType
		UE_REQUIRES(TIsContainerElementTypeCopyable_V<AliasElementType>)
	>
	UE_TCOMPACT_SET& operator=(UE_TCOMPACT_SET<typename TContainerElementTypeCompatibility<ElementType>::CopyFromOtherType, OtherKeyFuncs, Allocator> && Other)
	{
		TContainerElementTypeCompatibility<ElementType>::CopyingFromOtherType();
		Reset();
		Append(MoveTemp(Other));
		return *this;
	}

	/**
	 * Assignment operator. First deletes all currently contained elements
	 * and then copies from other set.
	 * Compatible element type version.
	 *
	 * @param Other The source set to assign from.
	 */
	template <
		typename OtherKeyFuncs,
		typename OtherAllocator,
		typename AliasElementType = ElementType
		UE_REQUIRES(TIsContainerElementTypeCopyable_V<AliasElementType>)
	>
	UE_TCOMPACT_SET& operator=(const UE_TCOMPACT_SET<typename TContainerElementTypeCompatibility<ElementType>::CopyFromOtherType, OtherKeyFuncs, OtherAllocator>& Other)
	{
		TContainerElementTypeCompatibility<ElementType>::CopyingFromOtherType();
		Reset();
		Append(Other);
		return *this;
	}

	/**
	 * Removes all elements from the set, potentially leaving space allocated for an expected number of elements about to be added.
	 * @param ExpectedNumElements - The number of elements about to be added to the set.
	 */
	void Empty(int32 ExpectedNumElements = 0)
	{
		DestructItems(GetData(), this->NumElements);
		this->NumElements = 0;

		ResizeAllocation(ExpectedNumElements);

		if (this->MaxElements > 0)
		{
			GetHashTableView().Reset();
		}
	}

	/** Efficiently empties out the set but preserves all allocations and capacities */
	void Reset()
	{
		if (this->NumElements > 0)
		{
			FCompactHashTableView HashTable = GetHashTableView();

			#if UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE
			if (this->NumElements < (SizeType)(HashTable.GetHashCount() / UE_SET_UNHASH_THRESHOLD_STORED_KEY))
			{
				for (int32 Index = 0; Index < this->NumElements; ++Index)
				{
					HashTable.RemoveHashElement(HashTable.GetStoredKey(Index, this->NumElements));
				}
			}
			#else
			if (this->NumElements < (SizeType)(HashTable.GetHashCount() / UE_SET_UNHASH_THRESHOLD_CALCULATE_KEY))
			{
				// if we've only used a portion of the full allocated size then it can be faster
				// to only clear out the used hash elements instead of memsetting the whole of the memory
				// The exact number can be hard to determine since we have to recalculate the hash value to clear it

				for (const ElementType& Element : *this)
				{
					const KeyInitType Key = KeyFuncs::GetSetKey(Element);
					const uint32 KeyHash = KeyFuncs::GetKeyHash(Key);

					HashTable.RemoveHashElement(KeyHash);
				}
			}
			#endif
			else
			{
				HashTable.Reset();
			}

			DestructItems(GetData(), this->NumElements);
			this->NumElements = 0;
		}
	}

	/** Shrinks the set's element storage to avoid slack. */
	void Shrink()
	{
		if (this->NumElements != this->MaxElements)
		{
			if (ResizeAllocationPreserveData(this->NumElements))
			{
				Rehash();
			}
		}
	}

	/** Preallocates enough memory to contain Number elements */
	void Reserve(int32 Number)
	{
		// makes sense only when Number > Elements.Num() since TSparseArray::Reserve 
		// does any work only if that's the case
		if ((USizeType)Number > (USizeType)this->MaxElements)
		{
			// Trap negative reserves
			if (Number < 0)
			{
				UE::Core::Private::OnInvalidSetNum((unsigned long long)Number);
			}

			if (ResizeAllocationPreserveData(Number))
			{
				Rehash();
			}
		}
	}

	/** Deprecated */
	void Compact()
	{
		// Data is always compacted, but this is useful to be able to more easily swap between sparse and compact sets
		// We can at least perform a shrink here to maintain memory parity with sparse set
		Shrink();
	}

	/** 
	 * Helper function to return the amount of memory allocated by this container 
	 * Only returns the size of allocations made directly by the container, not the elements themselves.
	 * @return number of bytes allocated by this container
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT SIZE_T GetAllocatedSize() const
	{
		return Super::GetAllocatedSize(GetSetLayout());
	}

	/** Tracks the container's memory use through an archive. */
	inline void CountBytes(FArchive& Ar) const
	{
		const FCompactSetLayout Layout = GetSetLayout();
		Ar.CountBytes(Super::GetTotalMemoryRequiredInBytes(this->NumElements, Layout), Super::GetTotalMemoryRequiredInBytes(this->MaxElements, Layout));
	}

	/* Calculate the size of the hash table from the number of elements in the set assuming the default number of hash elements */
	[[nodiscard]] UE_FORCEINLINE_HINT static constexpr size_t GetTotalMemoryRequiredInBytes(uint32 NumElements)
	{
		return Super::GetTotalMemoryRequiredInBytes(NumElements, GetSetLayout());
	}

	/**
	 * Checks whether an element id is valid.
	 * @param Id - The element id to check.
	 * @return true if the element identifier refers to a valid element in this set.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValidId(FSetElementId Id) const
	{
		return Id.AsInteger() >= 0 && Id.AsInteger() < this->NumElements;
	}

	/**
	 * Checks array invariants: if array size is greater than or equal to zero and less
	 * than or equal to the maximum.
	 */
	UE_FORCEINLINE_HINT void CheckInvariants() const
	{
		checkSlow((this->NumElements >= 0) & (this->MaxElements >= this->NumElements)); // & for one branch
	}

	/**
	 * Checks if index is in array range.
	 *
	 * @param Index Index to check.
	 */
	inline void RangeCheck(FSetElementId Id) const
	{
		CheckInvariants();

		// Template property, branch will be optimized out
		if constexpr (AllocatorType::RequireRangeCheck)
		{
			checkf(IsValidId(Id), TEXT("Array index out of bounds: %d into an array of size %d"), Id.AsInteger(), this->NumElements);
		}
	}

	/** Accesses the identified element's value. Element must be valid (see @IsValidId). */
	[[nodiscard]] inline ElementType & operator[](FSetElementId Id)
	{
		RangeCheck(Id);
		return GetData()[Id.AsInteger()];
	}

	/** Accesses the identified element's value. Element must be valid (see @IsValidId). */
	[[nodiscard]] inline const ElementType & operator[](FSetElementId Id) const
	{
		RangeCheck(Id);
		return GetData()[Id.AsInteger()];
	}

	/** Accesses the identified element's value. Element must be valid (see @IsValidId). */
	[[nodiscard]] inline ElementType& Get(FSetElementId Id)
	{
		RangeCheck(Id);
		return GetData()[Id.AsInteger()];
	}

	/** Accesses the identified element's value. Element must be valid (see @IsValidId). */
	[[nodiscard]] inline const ElementType& Get(FSetElementId Id) const
	{
		RangeCheck(Id);
		return GetData()[Id.AsInteger()];
	}

	/**
	 * Adds an element to the set.
	 *
	 * @param	InElement					Element to add to set
	 * @param	bIsAlreadyInSetPtr	[out]	Optional pointer to bool that will be set depending on whether element is already in set
	 * @return	A pointer to the element stored in the set.
	 */
	UE_FORCEINLINE_HINT FSetElementId Add(const ElementType&  InElement, bool* bIsAlreadyInSetPtr = nullptr)
	{
		return Emplace(InElement, bIsAlreadyInSetPtr);
	}
	UE_FORCEINLINE_HINT FSetElementId Add(ElementType && InElement, bool* bIsAlreadyInSetPtr = nullptr)
	{
		return Emplace(MoveTempIfPossible(InElement), bIsAlreadyInSetPtr);
	}

	/**
	 * Adds an element to the set.
	 *
	 * @see		Class documentation section on ByHash() functions
	 * @param	KeyHash						A precomputed hash value, calculated in the same way as ElementType is hashed.
	 * @param	InElement					Element to add to set
	 * @param	bIsAlreadyInSetPtr	[out]	Optional pointer to bool that will be set depending on whether element is already in set
     * @return  A handle to the element stored in the set
     */
	UE_FORCEINLINE_HINT FSetElementId AddByHash(uint32 KeyHash, const ElementType& InElement, bool* bIsAlreadyInSetPtr = nullptr)
	{
		return EmplaceByHash(KeyHash, InElement, bIsAlreadyInSetPtr);
	}
	UE_FORCEINLINE_HINT FSetElementId AddByHash(uint32 KeyHash,		 ElementType && InElement, bool* bIsAlreadyInSetPtr = nullptr)
	{
		return EmplaceByHash(KeyHash, MoveTempIfPossible(InElement), bIsAlreadyInSetPtr);
	}

	/**
	 * Adds an element to the set.
	 *
	 * @param	Arg							The argument(s) to be forwarded to the set element's constructor.
	 * @param	bIsAlreadyInSetPtr	[out]	Optional pointer to bool that will be set depending on whether element is already in set
	 * @return	A handle to the element stored in the set.
	 */
	template<typename ArgType = ElementType>
	UE_FORCEINLINE_HINT FSetElementId Emplace(ArgType && Arg, bool* bIsAlreadyInSetPtr = nullptr)
	{
		TPair<FSetElementId, bool> Result = Emplace(InPlace, Forward<ArgType>(Arg));

		if (bIsAlreadyInSetPtr)
		{
			*bIsAlreadyInSetPtr = Result.Value;
		}

		return Result.Key;
	}

    /**
     * Adds an element to the set by constructing the ElementType in-place with multiple args.
     *
     * @param   EInPlace   Tag to disambiguate in-place construction.
     * @param   InArgs     Arguments forwarded to ElementType's constructor.
     * @return  Pair of (element id, whether an equivalent element already existed).
     */
    template <typename... ArgTypes>
    TPair<FSetElementId, bool> Emplace(EInPlace, ArgTypes&&... InArgs)
    {
		alignas(alignof(ElementType))char StackElement[sizeof(ElementType)];
		ElementType& TempElement = *new (StackElement) ElementType(Forward<ArgTypes>(InArgs)...);
		SizeType ExistingIndex(INDEX_NONE);

		const KeyInitType Key = KeyFuncs::GetSetKey(TempElement);
		const uint32 KeyHash = KeyFuncs::GetKeyHash(Key);

		if constexpr (!KeyFuncs::bAllowDuplicateKeys)
		{
			ExistingIndex = FindIndexByHash(KeyHash, Key);
		}

		const bool bAlreadyInSet = ExistingIndex != INDEX_NONE;

		if (!bAlreadyInSet)
		{
			ExistingIndex = this->NumElements;
			ElementType& NewElement = AddUninitialized(KeyHash);
			RelocateConstructItem<ElementType, ElementType, SizeType>(&NewElement, &TempElement);
		}
		else
		{
			ElementType& ExistingElement = GetData()[ExistingIndex];
			MoveByRelocate(ExistingElement, TempElement);
		}

		return { FSetElementId::FromInteger(ExistingIndex), bAlreadyInSet };
	}
	
	/**
	 * Adds an element to the set.
	 *
	 * @see		Class documentation section on ByHash() functions
	 * @param	KeyHash						A precomputed hash value, calculated in the same way as ElementType is hashed.
	 * @param	Arg							The argument(s) to be forwarded to the set element's constructor.
	 * @param	bIsAlreadyInSetPtr	[out]	Optional pointer to bool that will be set depending on whether element is already in set
	 * @return	A handle to the element stored in the set.
	 */
	template<typename ArgType = ElementType>
	UE_FORCEINLINE_HINT FSetElementId EmplaceByHash(uint32 KeyHash, ArgType && Arg, bool* bIsAlreadyInSetPtr = nullptr)
	{
		TPair<FSetElementId, bool> Result = EmplaceByHash(InPlace, KeyHash, Forward<ArgType>(Arg));

		if (bIsAlreadyInSetPtr)
		{
			*bIsAlreadyInSetPtr = Result.Value;
		}

		return Result.Key;
	}

    /**
     * Adds an element to the set by constructing in-place with multiple args, using a precomputed hash.
	 *
     * @see     Class documentation section on ByHash() functions.
	 * @param	KeyHash    A precomputed hash value, calculated in the same way as ElementType is hashed.
     * @param   EInPlace   Tag to disambiguate in-place construction.
     * @param   InArgs     Arguments forwarded to ElementType's constructor.
     * @return  Pair of (element id, whether an equivalent element already existed).
     */
    template <typename... ArgTypes>
    TPair<FSetElementId, bool> EmplaceByHash(EInPlace, uint32 KeyHash, ArgTypes&&... InArgs)
    {
		alignas(alignof(ElementType))char StackElement[sizeof(ElementType)];
		ElementType& TempElement = *new (StackElement) ElementType(Forward<ArgTypes>(InArgs)...);
		SizeType ExistingIndex(INDEX_NONE);

		if constexpr (!KeyFuncs::bAllowDuplicateKeys)
		{
			ExistingIndex = FindIndexByHash(KeyHash, KeyFuncs::GetSetKey(TempElement));
		}

		const bool bAlreadyInSet = ExistingIndex != INDEX_NONE;

		if (!bAlreadyInSet)
		{
			ExistingIndex = this->NumElements;
			ElementType& NewElement = AddUninitialized(KeyHash);
			RelocateConstructItem<ElementType, ElementType, SizeType>(&NewElement, &TempElement);
		}
		else
		{
			ElementType& ExistingElement = GetData()[ExistingIndex];
			MoveByRelocate(ExistingElement, TempElement);
		}

		return { FSetElementId::FromInteger(ExistingIndex), bAlreadyInSet };
	}

	/**
	 * Adds an element to the set if not already present and returns a reference to the added or existing element.
	 *
	 * @param	InElement					Element to add to set
	 * @param	bIsAlreadyInSetPtr	[out]	Optional pointer to bool that will be set depending on whether element is already in set
	 * @return	A reference to the element stored in the set.
	 */
	UE_FORCEINLINE_HINT ElementType& FindOrAdd(const InElementType& InElement, bool* bIsAlreadyInSetPtr = nullptr)
	{
		return FindOrAddByHash(KeyFuncs::GetKeyHash(KeyFuncs::GetSetKey(InElement)), InElement, bIsAlreadyInSetPtr);
	}
	UE_FORCEINLINE_HINT ElementType& FindOrAdd(InElementType && InElement, bool* bIsAlreadyInSetPtr = nullptr)
	{
		const uint32 KeyHash = KeyFuncs::GetKeyHash(KeyFuncs::GetSetKey(InElement));
		return FindOrAddByHash(KeyHash, MoveTempIfPossible(InElement), bIsAlreadyInSetPtr);
	}

	/**
	 * Adds an element to the set if not already present and returns a reference to the added or existing element.
	 *
	 * @see		Class documentation section on ByHash() functions
	 * @param	KeyHash						A precomputed hash value, calculated in the same way as ElementType is hashed.
	 * @param	InElement					Element to add to set
	 * @param	bIsAlreadyInSetPtr	[out]	Optional pointer to bool that will be set depending on whether element is already in set
	 * @return  A reference to the element stored in the set
	 */
	template<typename ElementReferenceType>
	ElementType& FindOrAddByHash(uint32 KeyHash, ElementReferenceType && InElement, bool* bIsAlreadyInSetPtr = nullptr)
	{
		const SizeType ExistingIndex = FindIndexByHash(KeyHash, KeyFuncs::GetSetKey(InElement));
		const bool bIsAlreadyInSet = ExistingIndex != INDEX_NONE;
		if (bIsAlreadyInSetPtr)
		{
			*bIsAlreadyInSetPtr = bIsAlreadyInSet;
		}
		if (bIsAlreadyInSet)
		{
			return GetData()[ExistingIndex];
		}

		ElementType& NewElement = AddUninitialized(KeyHash);
		return *(new ((void*)&NewElement) ElementType(Forward<ElementReferenceType>(InElement)));
	}

	void Append(TArrayView<const ElementType> InElements)
	{
		Reserve(this->NumElements + InElements.Num());
		for (const ElementType& Element : InElements)
		{
			Add(Element);
		}
	}

	template<typename ArrayAllocator>
	void Append(TArray<ElementType, ArrayAllocator> && InElements)
	{
		Reserve(this->NumElements + InElements.Num());
		for (ElementType& Element : InElements)
		{
			Add(MoveTempIfPossible(Element));
		}
		InElements.Reset();
	}

	/**
	 * Add all items from another set to our set (union without creating a new set)
	 * @param OtherSet - The other set of items to add.
	 */
	template<typename OtherAllocator>
	void Append(const UE_TCOMPACT_SET<ElementType, KeyFuncs, OtherAllocator>& OtherSet)
	{
		Reserve(this->NumElements + OtherSet.Num());
		for (const ElementType& Element : OtherSet)
		{
			Add(Element);
		}
	}

	template<typename OtherAllocator>
	void Append(UE_TCOMPACT_SET<ElementType, KeyFuncs, OtherAllocator> && OtherSet)
	{
		Reserve(this->NumElements + OtherSet.Num());
		for (ElementType& Element : OtherSet)
		{
			Add(MoveTempIfPossible(Element));
		}
		OtherSet.Reset();
	}

	void Append(std::initializer_list<ElementType> InitList)
	{
		Reserve(this->NumElements + (int32)InitList.size());
		for (const ElementType& Element : InitList)
		{
			Add(Element);
		}
	}

	/**
	 * Add all items from another set to our set (union without creating a new set)
	 * Compatible element type version.
	 * @param OtherSet - The other set of items to add.
	 */
	template <
		typename OtherKeyFuncs,
		typename OtherAllocator,
		typename AliasElementType = ElementType
		UE_REQUIRES(TIsContainerElementTypeCopyable_V<AliasElementType>)
	>
	void Append(const UE_TCOMPACT_SET<typename TContainerElementTypeCompatibility<ElementType>::CopyFromOtherType, OtherKeyFuncs, OtherAllocator>& OtherSet)
	{
		TContainerElementTypeCompatibility<ElementType>::CopyingFromOtherType();
		Reserve(this->NumElements + OtherSet.Num());
		for (const ElementType& Element : OtherSet)
		{
			Add(Element);
		}
	}

	/**
	 * Add all items from another set to our set (union without creating a new set)
	 * Compatible element type version.
	 * @param OtherSet - The other set of items to add.
	 */
	template <
		typename OtherKeyFuncs,
		typename AliasElementType = ElementType
		UE_REQUIRES(TIsContainerElementTypeCopyable_V<AliasElementType>)
	>
	void Append(UE_TCOMPACT_SET<typename TContainerElementTypeCompatibility<ElementType>::CopyFromOtherType, OtherKeyFuncs, Allocator> && OtherSet)
	{
		TContainerElementTypeCompatibility<ElementType>::CopyingFromOtherType();
		Reserve(this->NumElements + OtherSet.Num());
		for (ElementType& Element : OtherSet)
		{
			Add(MoveTempIfPossible(Element));
		}
		OtherSet.Reset();
	}

	/**
	 * Finds any element in the set and returns a pointer to it.
	 * Callers should not depend on particular patterns in the behaviour of this function.
	 * @return A pointer to an arbitrary element, or nullptr if the container is empty.
	 */
	[[nodiscard]] ElementType* FindArbitraryElement()
	{
		return (this->NumElements > 0) ? GetData() : nullptr;
	}
	[[nodiscard]] const ElementType* FindArbitraryElement() const
	{
		return const_cast<UE_TCOMPACT_SET*>(this)->FindArbitraryElement();
	}


	/**
	 * Finds an element with the given key in the set.
	 * @param Key - The key to search for.
	 * @return The id of the set element matching the given key, or the NULL id if none matches.
	 */
	[[nodiscard]] FSetElementId FindId(KeyInitType Key) const
	{
		return FSetElementId::FromInteger(FindIndexByHash(KeyFuncs::GetKeyHash(Key), Key));
	}

	/**
	 * Finds an element with a pre-calculated hash and a key that can be compared to KeyType
	 * @see	Class documentation section on ByHash() functions
	 * @return The element id that matches the key and hash or an invalid element id
	 */
	template<typename ComparableKey>
	[[nodiscard]] FSetElementId FindIdByHash(uint32 KeyHash, const ComparableKey& Key) const
	{
		checkSlow(KeyHash == KeyFuncs::GetKeyHash(Key));
		return FSetElementId::FromInteger(FindIndexByHash(KeyHash, Key));
	}

	/**
	 * Finds an element with the given key in the set.
	 * @param Key - The key to search for.
	 * @return A pointer to an element with the given key.  If no element in the set has the given key, this will return NULL.
	 */
	[[nodiscard]] inline ElementType* Find(KeyInitType Key)
	{
		SizeType ElementIndex = FindIndexByHash(KeyFuncs::GetKeyHash(Key), Key);
		if (ElementIndex != INDEX_NONE)
		{
			return GetData() + ElementIndex;
		}
		else
		{
			return nullptr;
		}
	}
	
	/**
	 * Finds an element with the given key in the set.
	 * @param Key - The key to search for.
	 * @return A const pointer to an element with the given key.  If no element in the set has the given key, this will return NULL.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType* Find(KeyInitType Key) const
	{
		return const_cast<UE_TCOMPACT_SET*>(this)->Find(Key);
	}

	/**
	 * Finds an element with a pre-calculated hash and a key that can be compared to KeyType.
	 * @see	Class documentation section on ByHash() functions
	 * @return A pointer to the contained element or nullptr.
	 */
	template<typename ComparableKey>
	[[nodiscard]] ElementType* FindByHash(uint32 KeyHash, const ComparableKey& Key)
	{
		SizeType ElementIndex = FindIndexByHash(KeyHash, Key);
		if (ElementIndex != INDEX_NONE)
		{
			return GetData() + ElementIndex;
		}
		else
		{
			return nullptr;
		}
	}

	template<typename ComparableKey>
	[[nodiscard]] const ElementType* FindByHash(uint32 KeyHash, const ComparableKey& Key) const
	{
		return const_cast<UE_TCOMPACT_SET*>(this)->FindByHash(KeyHash, Key);
	}

	/**
	 * Removes an element from the set.
	 * @param Element - A pointer to the element in the set, as returned by Add or Find.
	 */
	void Remove(FSetElementId ElementId)
	{
		RemoveByIndex(ElementId.AsInteger());
	}

	/**
	 * Removes all elements from the set matching the specified key.
	 * @param Key - The key to match elements against.
	 * @return The number of elements removed.
	 */
	int32 Remove(KeyInitType Key)
	{
		if (this->NumElements)
		{
			return RemoveImpl(KeyFuncs::GetKeyHash(Key), Key);
		}

		return 0;
	}

	/**
	 * Removes an element from the set while maintaining set order.
	 * @param Element - A pointer to the element in the set, as returned by Add or Find.
	 */
	void RemoveStable(FSetElementId ElementId)
	{
		RemoveByIndex<true>(ElementId.AsInteger());
	}

	/**
	 * Removes all elements from the set matching the specified key.
	 * @param Key - The key to match elements against.
	 * @return The number of elements removed.
	 */
	int32 RemoveStable(KeyInitType Key)
	{
		if (this->NumElements)
		{
			return RemoveImplStable(KeyFuncs::GetKeyHash(Key), Key);
		}

		return 0;
	}

	/**
	 * Removes all elements from the set matching the specified key.
	 *
	 * @see		Class documentation section on ByHash() functions
	 * @param	Key - The key to match elements against.
	 * @return	The number of elements removed.
	 */
	template<typename ComparableKey>
	int32 RemoveByHash(uint32 KeyHash, const ComparableKey& Key)
	{
		checkSlow(KeyHash == KeyFuncs::GetKeyHash(Key));

		if (this->NumElements)
		{
			return RemoveImpl(KeyHash, Key);
		}

		return 0;
	}

	/**
	 * Checks if the element contains an element with the given key.
	 * @param Key - The key to check for.
	 * @return true if the set contains an element with the given key.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool Contains(KeyInitType Key) const
	{
		return FindIndexByHash(KeyFuncs::GetKeyHash(Key), Key) != INDEX_NONE;
	}

	/**
	 * Checks if the element contains an element with the given key.
	 *
	 * @see	Class documentation section on ByHash() functions
	 */
	template<typename ComparableKey>
	[[nodiscard]] inline bool ContainsByHash(uint32 KeyHash, const ComparableKey& Key) const
	{
		checkSlow(KeyHash == KeyFuncs::GetKeyHash(Key));
		return FindIndexByHash(KeyHash, Key) != INDEX_NONE;
	}

	/**
	 * Sorts the set's elements using the provided comparison class.
	 */
	template<typename PredicateType>
	void Sort(const PredicateType& Predicate)
	{
		TArrayView<ElementType>(GetData(), this->NumElements).Sort(Predicate);
		Rehash();
	}

	/**
	 * Stable sorts the set's elements using the provided comparison class.
	 */
	template<typename PredicateType>
	void StableSort(const PredicateType& Predicate)
	{
		TArrayView<ElementType>(GetData(), this->NumElements).StableSort(Predicate);
		Rehash();
	}

	/**
	 * Describes the set's contents through an output device.
	 * @param Ar - The output device to describe the set's contents through.
	 */
	void Dump(FOutputDevice& Ar) const
	{
		if (this->MaxElements == 0)
		{
			Ar.Log(TEXT("UE_TCOMPACT_SET: empty"));
		}
		else if (this->MaxElements > 0)
		{
			FConstCompactHashTableView HashTable = this->GetConstHashTableView();

			Ar.Logf(TEXT("UE_TCOMPACT_SET: %d elements, %d max elements, %d hash slots"), this->NumElements, this->MaxElements, HashTable.GetHashCount());

			for (uint32 HashIndex = 0; HashIndex < HashTable.GetHashCount(); ++HashIndex)
			{
				// Count the numTableIndexber of elements in this hash bucket.
				int32 NumElementsInBucket = 0;
				for (uint32 ElementIndex = HashTable.GetFirstByIndex(HashIndex); ElementIndex != INDEX_NONE; ElementIndex = HashTable.GetNext(ElementIndex, this->NumElements))
				{
					NumElementsInBucket++;
				}

				Ar.Logf(TEXT("   Hash[%i] = %i"), HashIndex, NumElementsInBucket);
			}
		}
		else
		{
			// MaxElements == INDEX_NONE is a TOptional that's null, anything else is just garbage data
			checkNoEntry();
		}
	}

	/** @return the intersection of two sets. (A AND B)*/
	[[nodiscard]] UE_TCOMPACT_SET Intersect(const UE_TCOMPACT_SET& OtherSet) const
	{
		const bool bOtherSmaller = (this->NumElements > OtherSet.NumElements);
		const UE_TCOMPACT_SET& A = (bOtherSmaller ? OtherSet : *this);
		const UE_TCOMPACT_SET& B = (bOtherSmaller ? *this : OtherSet);

		UE_TCOMPACT_SET Result;
		Result.Reserve(A.NumElements); // Worst case is everything in smaller is in larger

		for (const ElementType& Element : A)
		{
			if (B.Contains(KeyFuncs::GetSetKey(Element)))
			{
				Result.Add(Element);
			}
		}
		return Result;
	}

	/** @return the union of two sets. (A OR B)*/
	[[nodiscard]] UE_TCOMPACT_SET Union(const UE_TCOMPACT_SET& OtherSet) const
	{
		UE_TCOMPACT_SET Result;
		Result.Reserve(this->NumElements + OtherSet.NumElements); // Worst case is 2 totally unique Sets

		for (const ElementType& Element : *this)
		{
			Result.Add(Element);
		}
		for (const ElementType& Element : OtherSet)
		{
			Result.Add(Element);
		}
		return Result;
	}

	/** @return the complement of two sets. (A not in B where A is this and B is Other)*/
	[[nodiscard]] UE_TCOMPACT_SET Difference(const UE_TCOMPACT_SET& OtherSet) const
	{
		UE_TCOMPACT_SET Result;
		Result.Reserve(this->NumElements); // Worst case is no elements of this are in Other

		for (const ElementType& Element : *this)
		{
			if (!OtherSet.Contains(KeyFuncs::GetSetKey(Element)))
			{
				Result.Add(Element);
			}
		}
		return Result;
	}
	
	/** @return the symmetric difference of two sets. (A not in B and B not in A where A is this and B is Other) */
	[[nodiscard]] UE_TCOMPACT_SET SymmetricDifference(const UE_TCOMPACT_SET& OtherSet) const
	{
		UE_TCOMPACT_SET Result;
		Result.Reserve(this->NumElements + OtherSet.NumElements); // Worst case: no overlap

		// Add all elements from A that are not in B
		for (const ElementType& Element : *this)
		{
			if (!OtherSet.Contains(KeyFuncs::GetSetKey(Element)))
			{
				Result.Add(Element);
			}
		}

		// Add all elements from B that are not in A
		for (const ElementType& Element : OtherSet)
		{
			if (!this->Contains(KeyFuncs::GetSetKey(Element)))
			{
				Result.Add(Element);
			}
		}

		return Result;
	}

	/**
	 * Determine whether the specified set is entirely included within this set
	 * 
	 * @param OtherSet	Set to check
	 * 
	 * @return True if the other set is entirely included in this set, false if it is not
	 */
	[[nodiscard]] bool Includes(const UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>& OtherSet) const
	{
		bool bIncludesSet = true;
		if (OtherSet.NumElements <= this->NumElements)
		{
			for (const ElementType& Element : OtherSet)
			{
				if (!Contains(KeyFuncs::GetSetKey(Element)))
				{
					bIncludesSet = false;
					break;
				}
			}
		}
		else
		{
			// Not possible to include if it is bigger than us
			bIncludesSet = false;
		}
		return bIncludesSet;
	}

	/** @return a TArray that's a copy of all the elements in this set, prefer using ArrayView if available. */
	[[nodiscard]] TArray<ElementType> Array() const
	{
		return TArray<ElementType>(GetData(), this->NumElements);
	}

#if UE_TCOMPACT_SET_ARRAY_VIEW_ENABLED
	/** @return a readonly TArrayView of the elements */
	[[nodiscard]] TArrayView<const ElementType> ArrayView() const
	{
		return TArrayView<const ElementType>(GetData(), this->NumElements);
	}
#endif

	/** Creates an iterator for the contents of this set */
	[[nodiscard]] UE_FORCEINLINE_HINT TIterator CreateIterator()
	{
		return TIterator(*this);
	}

	/** Creates a const iterator for the contents of this set */
	[[nodiscard]] UE_FORCEINLINE_HINT TConstIterator CreateConstIterator() const
	{
		return TConstIterator(*this);
	}

	/**
	* DO NOT USE DIRECTLY
	* STL-like iterators to enable range-based for loop support.
	*/
	#if TARRAY_RANGED_FOR_CHECKS // Only use iterators with checks enabled to get modification check, otherwise it's just a waste of compile/runtime resources
	using TRangedForIterator             = TCheckedPointerIterator<      ElementType, SizeType, false>;
	using TRangedForConstIterator        = TCheckedPointerIterator<const ElementType, SizeType, false>;

	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForIterator      begin() { return TRangedForIterator(this->NumElements, GetData()); }
	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForConstIterator begin() const { return TRangedForConstIterator(this->NumElements, GetData()); }
	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForIterator      end() { return TRangedForIterator(this->NumElements, GetData() + this->NumElements); }
	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForConstIterator end() const { return TRangedForConstIterator(this->NumElements, GetData() + this->NumElements); }
	#else
	using TRangedForIterator = ElementType*;
	using TRangedForConstIterator = const ElementType*;

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType*       begin() { return GetData(); }
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType* begin() const { return GetData(); }
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType*       end() { return GetData() + this->NumElements; }
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType* end() const { return GetData() + this->NumElements; }
	#endif

	// Sets are deliberately prevented from being hashed or compared, because this would hide potentially major performance problems behind default operations.
	friend uint32 GetTypeHash(const UE_TCOMPACT_SET& Set) = delete;
	friend bool operator==(const UE_TCOMPACT_SET&, const UE_TCOMPACT_SET&) = delete;
	friend bool operator!=(const UE_TCOMPACT_SET&, const UE_TCOMPACT_SET&) = delete;

	void WriteMemoryImage(FMemoryImageWriter& Writer) const
	{
		checkf(!Writer.Is32BitTarget(), TEXT("UE_TCOMPACT_SET does not currently support freezing for 32bits"));
		if constexpr (TAllocatorTraits<Allocator>::SupportsFreezeMemoryImage && THasTypeLayout<InElementType>::Value)
		{
			if (this->NumElements > 0)
			{
				const ElementType* Data = GetData();
				check(Data);
				FMemoryImageWriter ArrayWriter = Writer.WritePointer(StaticGetTypeLayoutDesc<ElementType>());
				ArrayWriter.WriteAlignment(GetAlignment());

				// Write active element data
				ArrayWriter.WriteObjectArray(Data, StaticGetTypeLayoutDesc<ElementType>(), this->NumElements);
				ArrayWriter.WritePaddingToSize(this->MaxElements * sizeof(ElementType));

				// Write remaining byte and hash table data
				const FCompactSetLayout Layout = GetSetLayout();
				const HashCountType* HashTable = this->GetHashTableMemory(Layout);
				const uint8* HashTableDataEnd = (const uint8*)Data + Super::GetTotalMemoryRequiredInBytes(this->MaxElements, *HashTable, Layout);
				ArrayWriter.WriteBytes(HashTable, HashTableDataEnd - (const uint8*)HashTable);

				Writer.WriteBytes(this->NumElements);
				Writer.WriteBytes(this->MaxElements);
			}
			else
			{
				Writer.WriteBytes(UE_TCOMPACT_SET());
			}
		}
		else
		{
			Writer.WriteBytes(UE_TCOMPACT_SET());
		}
	}

	void CopyUnfrozen(const FMemoryUnfreezeContent& Context, void* Dst) const
	{
		::new(Dst) UE_TCOMPACT_SET();

		if constexpr (TAllocatorTraits<Allocator>::SupportsFreezeMemoryImage && THasTypeLayout<InElementType>::Value)
		{
			const FTypeLayoutDesc& ElementTypeDesc = StaticGetTypeLayoutDesc<ElementType>();

			UE_TCOMPACT_SET* DstObject = static_cast<UE_TCOMPACT_SET*>(Dst);

			{
				DstObject->ResizeAllocation(this->MaxElements);
				DstObject->NumElements = this->NumElements;

				const ElementType* SrcData = GetData();
				ElementType* DstData = DstObject->GetData();

				for (int32 Index = 0; Index < this->NumElements; ++Index)
				{
					Context.UnfreezeObject(SrcData + Index, ElementTypeDesc, DstData + Index);
				}

				const FCompactSetLayout Layout = GetSetLayout();
				const HashCountType* SrcHashTable = this->GetHashTableMemory(Layout);
				const uint8* SrcHashTableEnd = (const uint8 *)SrcData + Super::GetTotalMemoryRequiredInBytes(this->MaxElements, *SrcHashTable, Layout);

				HashCountType* DstHashTable = (HashCountType*)DstObject->GetHashTableMemory(Layout);
				FMemory::Memcpy(DstHashTable, SrcHashTable, SrcHashTableEnd - (const uint8*)SrcHashTable);
			}
		}
	}

	static void AppendHash(const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		if constexpr (TAllocatorTraits<Allocator>::SupportsFreezeMemoryImage && THasTypeLayout<ElementType>::Value)
		{
			Freeze::AppendHash(StaticGetTypeLayoutDesc<ElementType>(), LayoutParams, Hasher);
		}
	}

private:
	/* Get the alignment required for the allocation */
	[[nodiscard]] UE_FORCEINLINE_HINT static constexpr size_t GetAlignment()
	{
		return FGenericPlatformMath::Max<size_t>(alignof(ElementType), UE::Core::CompactHashTable::GetMemoryAlignment());
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType* GetData()
	{
		return (ElementType*)this->Elements.GetAllocation();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType* GetData() const
	{
		return (const ElementType*)this->Elements.GetAllocation();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT static constexpr FCompactSetLayout GetSetLayout()
	{
		return { sizeof(ElementType), GetAlignment() };
	}

	[[nodiscard]] UE_FORCEINLINE_HINT FCompactHashTableView GetHashTableView()
	{
		return Super::GetHashTableView(GetSetLayout());
	}

	[[nodiscard]] UE_FORCEINLINE_HINT FConstCompactHashTableView GetConstHashTableView() const
	{
		return Super::GetConstHashTableView(GetSetLayout());
	}

	// Use if you're going to reset/rehash regardless of the results
	UE_FORCEINLINE_HINT void ResizeAllocation(SizeType NewMaxElements)
	{
		(void)Super::ResizeAllocationPreserveData(NewMaxElements, GetSetLayout(), false);
	}

	// Use this if you'll be keeping the element data
	[[nodiscard]] bool ResizeAllocationPreserveData(SizeType NewMaxElements, bool bPreserveHashData = true)
	{
		return Super::ResizeAllocationPreserveData(NewMaxElements, GetSetLayout(), bPreserveHashData);
	}
	
	/** Recalculate the lookup table, used after bulk operations are performed */
	void Rehash()
	{
		if (this->MaxElements > 0)
		{
			const ElementType* ElementData = GetData();
			FCompactHashTableView HashTable = GetHashTableView();

			HashTable.Reset();

			for (int32 Index = 0; Index < this->NumElements; ++Index)
			{
				HashTable.Add(Index, KeyFuncs::GetKeyHash(KeyFuncs::GetSetKey(ElementData[Index])));
			}
		}
	}

	[[nodiscard]] ElementType& AddUninitialized(uint32 KeyHash)
	{
		checkSlow(this->MaxElements >= 0);
		if (this->NumElements == this->MaxElements)
		{
			Reserve(this->AllocatorCalculateSlackGrow(this->NumElements + 1, GetSetLayout()));
		}
		GetHashTableView().Add(this->NumElements, KeyHash);
		return GetData()[this->NumElements++];
	}

	/**
	 * Finds an element with a pre-calculated hash and a key that can be compared to KeyType
	 * @see	Class documentation section on ByHash() functions
	 * @return The element id that matches the key and hash or an invalid element id
	 */
	template<typename ComparableKey>
	[[nodiscard]] SizeType FindIndexByHash(uint32 KeyHash, const ComparableKey& Key) const
	{
		if (this->NumElements == 0)
		{
			return INDEX_NONE;
		}

		const ElementType* ElementData = GetData();
		const HashCountType* HashTable = this->GetHashTableMemory(GetSetLayout());
		const uint8* NextIndicesData = (const uint8*)(HashTable + 1);
		const uint32 HashCount = *HashTable;
		
		// Inlining this can save up to 40% perf compared to GetHashTableView().Find()
		#define UE_COMPACTHASHTABLE_EXECUTEBYTYPE(Type) \
				const Type* NextIndices = (const Type *)NextIndicesData; \
				const Type* HashIndices = NextIndices + this->MaxElements * this->KeyStorageMultiplier; \
				for (Type Index = HashIndices[KeyHash & (HashCount-1)]; Index != (Type)INDEX_NONE; Index = NextIndices[Index]) \
				{ \
					checkSlow((SizeType)Index < this->NumElements); \
						if (KeyFuncs::Matches(KeyFuncs::GetSetKey(ElementData[Index]), Key)) \
						{ \
							return Index; \
						} \
				}
		
		UE_COMPACTHASHTABLE_CALLBYTYPE(this->MaxElements)
		#undef UE_COMPACTHASHTABLE_EXECUTEBYTYPE

		return INDEX_NONE;
	}

	template <bool IsStable = false>
	void RemoveByIndex(const SizeType ElementIndex)
	{
		checkf(ElementIndex >= 0 && ElementIndex < this->NumElements, TEXT("Invalid ElementIndex (%d) passed to UE_TCOMPACT_SET::RemoveByIndex. NumElements=%d"), ElementIndex, this->NumElements);

		ElementType* ElementsData = GetData();
		FCompactHashTableView HashTable = GetHashTableView();

		#if UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE
		RemoveByIndexAndHash<IsStable>(ElementIndex, HashTable.GetStoredKey(ElementIndex, this->NumElements), ElementsData, HashTable);
		#else
		RemoveByIndexAndHash<IsStable>(ElementIndex, KeyFuncs::GetKeyHash(KeyFuncs::GetSetKey(ElementsData[ElementIndex])), ElementsData, HashTable);
		#endif
	}

	template <bool IsStable = false>
	SizeType RemoveByIndexAndHash(const SizeType ElementIndex, const uint32 KeyHash, ElementType* ElementsData, FCompactHashTableView& HashTable)
	{
		const SizeType LastElementIndex = this->NumElements - 1;
		SizeType NextIndex; 
		
		// Note: Destructing/moving elements for better debugability
		if (ElementIndex == LastElementIndex)
		{
			NextIndex = HashTable.Remove(ElementIndex, KeyHash, ElementIndex, KeyHash);
			
			ElementsData[LastElementIndex].~ElementType();
		}
		else if constexpr (IsStable)
		{
			NextIndex = HashTable.RemoveStable(ElementIndex, KeyHash);

			ElementsData[ElementIndex].~ElementType();
			RelocateConstructItems<ElementType>(ElementsData + ElementIndex, ElementsData + ElementIndex + 1, LastElementIndex - ElementIndex);
		}
		else
		{
			#if UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE
			const uint32 LastElementHash = HashTable.GetStoredKey(LastElementIndex, this->NumElements);
			#else
			const uint32 LastElementHash = KeyFuncs::GetKeyHash(KeyFuncs::GetSetKey(ElementsData[LastElementIndex]));
			#endif
			NextIndex = HashTable.Remove(ElementIndex, KeyHash, LastElementIndex, LastElementHash);
			
			MoveByRelocate(ElementsData[ElementIndex], ElementsData[LastElementIndex]);
		}

		--this->NumElements;

		return NextIndex;
	}

	template<typename ComparableKey, bool IsStable = false>
	int32 RemoveImpl(uint32 KeyHash, const ComparableKey& Key)
	{
		checkSlow(this->NumElements > 0);
		int32 NumRemovedElements = 0;

		ElementType* ElementsData = GetData();
		FCompactHashTableView HashTable = GetHashTableView();

		SizeType ElementIndex = HashTable.GetFirst(KeyHash);

		while (ElementIndex != INDEX_NONE)
		{
			checkf(ElementIndex >= 0 && ElementIndex < this->NumElements, TEXT("Invalid ElementIndex (%d) passed to UE_TCOMPACT_SET::RemoveByIndexAndHash. NumElements=%d"), ElementIndex, this->NumElements);
			if (KeyFuncs::Matches(KeyFuncs::GetSetKey(ElementsData[ElementIndex]), Key))
			{
				if constexpr (!KeyFuncs::bAllowDuplicateKeys)
				{
					RemoveByIndexAndHash<IsStable>(ElementIndex, KeyHash, ElementsData, HashTable);
					// If the hash disallows duplicate keys, we're done removing after the first matched key.
					return 1;
				}
				else
				{
					ElementIndex = RemoveByIndexAndHash<IsStable>(ElementIndex, KeyHash, ElementsData, HashTable);
					NumRemovedElements++;
				}
			}
			else
			{
				ElementIndex = HashTable.GetNext(ElementIndex, this->NumElements);
			}
		}

		return NumRemovedElements;
	}

	template<typename ComparableKey>
	UE_FORCEINLINE_HINT int32 RemoveImplStable(uint32 KeyHash, const ComparableKey& Key)
	{
		return RemoveImpl<ComparableKey, true>(KeyHash, Key);
	}
};

template<typename RangeType>
UE_TCOMPACT_SET(RangeType &&)->UE_TCOMPACT_SET< TElementType_T<RangeType> >;

namespace Freeze
{
	template<typename ElementType, typename KeyFuncs, typename Allocator>
	void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>& Object, const FTypeLayoutDesc&)
	{
		Object.WriteMemoryImage(Writer);
	}

	template<typename ElementType, typename KeyFuncs, typename Allocator>
	uint32 IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>& Object, void* OutDst)
	{
		Object.CopyUnfrozen(Context, OutDst);
		return sizeof(Object);
	}

	template<typename ElementType, typename KeyFuncs, typename Allocator>
	uint32 IntrinsicAppendHash(const UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>::AppendHash(LayoutParams, Hasher);
		return DefaultAppendHash(TypeDesc, LayoutParams, Hasher);
	}
}

DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT((template<typename ElementType, typename KeyFuncs, typename Allocator>), (UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>));

struct TSETPRIVATEFRIEND
{
	/** Serializer. */
	template<typename ElementType, typename KeyFuncs, typename Allocator>
	static FArchive& Serialize(FArchive& Ar, UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>& Set)
	{
		Set.CountBytes(Ar);

		int32 NumElements = Set.Num();
		Ar << NumElements;

		if (Ar.IsLoading())
		{
			// We can skip the reset on Empty and do it once at the end with Rehash
			DestructItems(Set.GetData(), Set.Num());
			Set.NumElements = 0;
			Set.ResizeAllocation(NumElements);

			ElementType* Data = Set.GetData();
			for (int32 ElementIndex = 0; ElementIndex < NumElements; ElementIndex++)
			{
				Ar << *::new((void*)(Data + ElementIndex)) ElementType;
			}

			Set.NumElements = NumElements;
			Set.Rehash();
		}
		else
		{
			for (ElementType& Element : Set)
			{
				Ar << Element;
			}

			// Modifying refs is similar to load, in that we need to rehash the values after serializing.
			// Ex: See FArchiveReplaceObjectRef, which is often used to patch references via serialization.
			// This is reflecting the same logic being used in TScriptSparseSet for now to ensure correctness.
			// @todo - Maybe this could be optimized later to only rehash when the elements are pointer types.
			if (Ar.IsModifyingWeakAndStrongReferences() && !Ar.IsSaving())
			{
				Set.Rehash();
			}
		}
		return Ar;
	}

	/** Structured archive serializer. */
	template<typename ElementType, typename KeyFuncs, typename Allocator>
	static void SerializeStructured(FStructuredArchive::FSlot Slot, UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>& Set)
	{
		int32 NumElements = Set.Num();
		FStructuredArchive::FArray Array = Slot.EnterArray(NumElements);

		FArchive& Ar = Slot.GetUnderlyingArchive();
		if (Ar.IsLoading())
		{
			// We can skip the reset on Empty and do it once at the end with Rehash
			DestructItems(Set.GetData(), Set.Num());
			Set.NumElements = 0;
			Set.ResizeAllocation(NumElements);

			ElementType* Data = Set.GetData();
			for (int32 ElementIndex = 0; ElementIndex < NumElements; ElementIndex++)
			{
				FStructuredArchive::FSlot ElementSlot = Array.EnterElement();
				ElementSlot << *::new((void*)(Data + ElementIndex)) ElementType;
			}

			Set.NumElements = NumElements;
			Set.Rehash();
		}
		else
		{
			for (ElementType& Element : Set)
			{
				FStructuredArchive::FSlot ElementSlot = Array.EnterElement();
				ElementSlot << Element;
			}

			// Modifying refs is similar to load, in that we need to rehash the values after serializing.
			// Ex: See FArchiveReplaceObjectRef, which is often used to patch references via serialization.
			// This is reflecting the same logic being used in TScriptSparseSet for now to ensure correctness.
			// @todo - Maybe this could be optimized later to only rehash when the elements are pointer types.
			if (Ar.IsModifyingWeakAndStrongReferences() && !Ar.IsSaving())
			{
				Set.Rehash();
			}
		}
	}

	// Legacy comparison operators.  Note that these also test whether the set's elements were added in the same order!
	template<typename ElementType, typename KeyFuncs, typename Allocator>
	[[nodiscard]] static bool LegacyCompareEqual(const UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>& A, const UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>& B)
	{
		return A.Num() == B.Num() && CompareItems(A.GetData(), B.GetData(), A.Num());
	}
};

/** Serializer. */
template<typename ElementType, typename KeyFuncs, typename Allocator>
FArchive& operator<<(FArchive& Ar, UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>& Set)
{
	return TSETPRIVATEFRIEND::Serialize(Ar, Set);
}

/** Structured archive serializer. */
template<typename ElementType, typename KeyFuncs, typename Allocator>
void operator<<(FStructuredArchive::FSlot& Ar, UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>& Set)
{
	TSETPRIVATEFRIEND::SerializeStructured(Ar, Set);
}

// Legacy comparison operators.  Note that these also test whether the set's elements were added in the same order!
template<typename ElementType, typename KeyFuncs, typename Allocator>
[[nodiscard]] bool LegacyCompareEqual(const UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>& A, const UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>& B)
{
	return TSETPRIVATEFRIEND::LegacyCompareEqual(A, B);
}
template<typename ElementType, typename KeyFuncs, typename Allocator>
[[nodiscard]] bool LegacyCompareNotEqual(const UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>& A, const UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>& B)
{
	return !TSETPRIVATEFRIEND::LegacyCompareEqual(A, B);
}

template <typename ElementType, typename KeyFuncs, typename Allocator>
inline constexpr bool TIsTSet_V<UE_TCOMPACT_SET<ElementType, KeyFuncs, Allocator>> = true;

#undef TSETPRIVATEFRIEND
#undef UE_TCOMPACT_SET_ARRAY_VIEW_ENABLED