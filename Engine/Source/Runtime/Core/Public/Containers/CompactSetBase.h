// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/CompactHashTable.h"
#include "Templates/AlignmentTemplates.h"

/** Used to describe the data layout of the contents of the set */
struct FCompactSetLayout
{
	int32 Size;
	int32 Alignment;
};

/** Base class for the compact set provides common functionality to manage the data using FCompactSetLayout to describe it */
template<typename Allocator>
class TCompactSetBase
{
public:
	using AllocatorType = Allocator;
	using SizeType = typename AllocatorType::SizeType;

	[[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState Tag) const
	{
		return MaxElements == INDEX_NONE;
	}

	/**
	 * Returns true if the sets is empty and contains no elements. 
	 *
	 * @returns True if the set is empty.
	 * @see Num
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return NumElements == 0;
	}

	/** @return the number of elements. */
	[[nodiscard]] UE_FORCEINLINE_HINT SizeType Num() const
	{
		return NumElements;
	}

	/** @return The number of elements the set can hold before reallocation. */
	[[nodiscard]] UE_FORCEINLINE_HINT SizeType Max() const
	{
		return MaxElements;
	}

	/** Deprecated - unnecessary, keeping this here so TCompactSet can be swapped with TSet without changing code */
	[[nodiscard]] UE_FORCEINLINE_HINT SizeType GetMaxIndex() const
	{
		return NumElements;
	}

	/** 
	 * Helper function to return the amount of memory allocated by this container 
	 * Only returns the size of allocations made directly by the container, not the elements themselves.
	 * @return number of bytes allocated by this container
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT SIZE_T GetAllocatedSize(const FCompactSetLayout Layout) const
	{
		if (MaxElements == 0)
		{
			return 0;
		}

		return GetTotalMemoryRequiredInBytes(MaxElements, *GetHashTableMemory(Layout), Layout);
	}

protected:
	using HashCountType = uint32;
	static constexpr size_t HashCountSize = sizeof(HashCountType);
	static constexpr uint8 KeyStorageMultiplier = UE::Core::Private::CompactSetAllocatorHelpers::KeyStorageMultiplier;

	using ElementAllocatorType = typename AllocatorType::template ForElementType<uint8>;

	static_assert(std::is_same_v<SizeType, int32>, "TCompactSet currently only supports 32-bit allocators");
	static_assert(sizeof(HashCountType) == UE::Core::CompactHashTable::GetMemoryAlignment(), "Hashtable alignment changed, need to make sure HashCountType is stored correctly");

	[[nodiscard]] UE_FORCEINLINE_HINT constexpr TCompactSetBase() = default;

	[[nodiscard]] explicit consteval TCompactSetBase(EConstEval)
	: Elements(ConstEval)
	, NumElements(0)
	, MaxElements(0)
	{
	}

	[[nodiscard]] explicit TCompactSetBase(FIntrusiveUnsetOptionalState Tag)
	: NumElements(0)
	, MaxElements(INDEX_NONE)
	{
	}

	/* Calculate the size of the hash table from the number of elements in the set */
	[[nodiscard]] UE_FORCEINLINE_HINT const HashCountType* GetHashTableMemory(const FCompactSetLayout Layout) const
	{
		return (const HashCountType*)(Elements.GetAllocation() + GetElementsSizeInBytes(MaxElements, Layout));
	}

	[[nodiscard]] inline FCompactHashTableView GetHashTableView(const FCompactSetLayout Layout)
	{
		checkSlow(MaxElements > 0);
		const HashCountType* HashTable = GetHashTableMemory(Layout);
		return FCompactHashTableView((uint8*)(HashTable + 1), MaxElements, *HashTable, UE::Core::CompactHashTable::GetMemoryRequiredInBytes(MaxElements, *HashTable));
	}

	[[nodiscard]] inline FConstCompactHashTableView GetConstHashTableView(const FCompactSetLayout Layout) const
	{
		checkSlow(MaxElements > 0);
		const HashCountType* HashTable = GetHashTableMemory(Layout);
		return FConstCompactHashTableView((const uint8*)(HashTable + 1), MaxElements, *HashTable, UE::Core::CompactHashTable::GetMemoryRequiredInBytes(MaxElements, *HashTable));
	}

	/* Calculate the size of the hash table from the number of elements in the set */
	[[nodiscard]] UE_FORCEINLINE_HINT static constexpr HashCountType GetHashCount(uint32 NumElements)
	{
		return UE::Core::CompactHashTable::GetHashCount(NumElements);
	}

	/* Calculate bytes required to store the elements, includes any padding required for the hash table */
	[[nodiscard]] UE_FORCEINLINE_HINT static constexpr size_t GetElementsSizeInBytes(uint32 NumElements, const FCompactSetLayout Layout)
	{
		return Align(Layout.Size * NumElements, UE::Core::CompactHashTable::GetMemoryAlignment());
	}

	/* Get the total memory required for the compact set for the given number of elements and size of the number of hash elements */
	[[nodiscard]] UE_FORCEINLINE_HINT static constexpr size_t GetTotalMemoryRequiredInBytes(uint32 NumElements, HashCountType HashCount, const FCompactSetLayout Layout)
	{
		return NumElements ? GetElementsSizeInBytes(NumElements, Layout) + UE::Core::CompactHashTable::GetMemoryRequiredInBytes(NumElements, HashCount) + HashCountSize : 0;
	}

	/* Calculate the size of the hash table from the number of elements in the set assuming the default number of hash elements */
	[[nodiscard]] UE_FORCEINLINE_HINT static constexpr size_t GetTotalMemoryRequiredInBytes(uint32 NumElements, const FCompactSetLayout Layout)
	{
		return NumElements ? GetElementsSizeInBytes(NumElements, Layout) + UE::Core::CompactHashTable::GetMemoryRequiredInBytes(NumElements, GetHashCount(NumElements)) + HashCountSize : 0;
	}

	/* The total amount of memory required by the has set to store N elements with a hash table size of Y */
	[[nodiscard]] static constexpr SizeType GetMaxElementsForAvailableSpace(size_t TotalBytes, HashCountType HashCount, uint32 MinElementCount, const FCompactSetLayout Layout)
	{
		// Given some space in memory and a requested size for hash table, figure out how many elements we can fit in the remaining space
		const uint32 TypeSize = UE::Core::CompactHashTable::GetTypeSize(MinElementCount);
		const uint32 TypeShift = UE::Core::CompactHashTable::GetTypeShift(MinElementCount);
		const size_t AvailableBytes = TotalBytes - sizeof(HashCountType) - ((size_t)HashCount << TypeShift); // Remove hashtable and HashCount data
		const size_t MaxElements = AvailableBytes / (Layout.Size + TypeSize * KeyStorageMultiplier); // Calculate the max available ignoring the fact that the hash data has to be aligned
		const size_t RealAvailableBytes = AlignDown(AvailableBytes - (MaxElements << TypeShift) * KeyStorageMultiplier, UE::Core::CompactHashTable::GetMemoryAlignment()); // Remove the max required indexes and align down
		return (SizeType)FMath::Min<size_t>(MaxElements, RealAvailableBytes / Layout.Size); // Now we can get the true number of elements we could potentially use within the aligned space
	}

	[[nodiscard]] SizeType AllocatorCalculateSlackGrow(SizeType NewMaxElements, const FCompactSetLayout& Layout) const
	{
		const HashCountType OldHashCount = MaxElements > 0 ? *(HashCountType*)(Elements.GetAllocation() + GetElementsSizeInBytes(MaxElements, Layout)) : 0;
		const size_t OldSize = GetTotalMemoryRequiredInBytes(MaxElements, OldHashCount, Layout);

		const HashCountType NewHashCount = GetHashCount(NewMaxElements);
		const size_t NewSize = GetTotalMemoryRequiredInBytes(NewMaxElements, NewHashCount, Layout);
		size_t NewSlackSize = 0;

		checkf(NewSize > 0 && NewSize <= MAX_int32, TEXT("Invalid size for TSet[%zd]: NewMaxElements[%d] ElementSize[%d] HashCount[%d]"), NewSize, NewMaxElements, Layout.Size, NewHashCount);

		if constexpr (TAllocatorTraits<AllocatorType>::SupportsElementAlignment)
		{
			NewSlackSize = Elements.CalculateSlackGrow((SizeType)NewSize, (SizeType)OldSize, 1, Layout.Alignment);
		}
		else
		{
			NewSlackSize = Elements.CalculateSlackGrow((SizeType)NewSize, (SizeType)OldSize, 1);
		}

		if (NewSlackSize == NewSize)
		{
			// No need for expensive slack calculations if there is none
			return NewMaxElements;
		}

		// Calculate the number of items we can fit in the allotted space
		SizeType SlackNumElements = GetMaxElementsForAvailableSpace(NewSlackSize, NewHashCount, NewMaxElements, Layout);
		if (SlackNumElements <= NewMaxElements)
		{
			// This can happen if we're basically at the slack limit, when we got to calculate the results, even adding one element tracking index can
			// cause us to lose space due to alignment changes, in this case we just take the user input as the source of truth
			return NewMaxElements;
		}

		HashCountType SlackHashCount = GetHashCount(SlackNumElements);
		if (SlackHashCount > NewHashCount)
		{
			// New Size required to much space. Cut the new hash count in half and restrict the elements to the max for that hash count
			SlackNumElements = SlackHashCount - 1;
			SlackHashCount /= 2;
		}

		checkSlow(SlackNumElements >= NewMaxElements);
		checkSlow(GetTotalMemoryRequiredInBytes(SlackNumElements, SlackHashCount, Layout) <= NewSlackSize);

		return SlackNumElements;
	}

	void ResizeAllocation(const SizeType NewMaxElements, const FCompactSetLayout& Layout)
	{
		(void)ResizeAllocationPreserveData(NewMaxElements, Layout, false);
	}

	// Use this is you'll be keeping the element data
	[[nodiscard]] bool ResizeAllocationPreserveData(const SizeType NewMaxElements, const FCompactSetLayout& Layout, bool bPreserve = true)
	{
		bool bRequiresRehash = false;

		if (NewMaxElements != MaxElements)
		{
			const HashCountType OldHashCount = MaxElements > 0 ? *(HashCountType*)(Elements.GetAllocation() + GetElementsSizeInBytes(MaxElements, Layout)) : 0;
			const size_t OldSize = MaxElements > 0 ? GetTotalMemoryRequiredInBytes(MaxElements, OldHashCount, Layout) : 0;

			const HashCountType NewHashCount = NewMaxElements > 0 ? UE::Core::CompactHashTable::GetHashCount(NewMaxElements) : 0;
			const size_t NewSize = NewMaxElements > 0 ? GetTotalMemoryRequiredInBytes(NewMaxElements, NewHashCount, Layout) : 0;

			checkf(NewSize >= 0 && NewSize <= MAX_int32, TEXT("Invalid size for TSet[%lld]: NewMaxElements[%d] ElementSize[%d] HashCount[%d]"), NewSize, NewMaxElements, Layout.Size, NewHashCount);

			if (bPreserve && NewMaxElements > MaxElements && OldHashCount == NewHashCount)
			{
				// If preserving then we copy over all the data
				if constexpr (TAllocatorTraits<AllocatorType>::SupportsElementAlignment)
				{
					Elements.ResizeAllocation((SizeType)OldSize, (SizeType)NewSize, 1, Layout.Alignment);
				}
				else
				{
					Elements.ResizeAllocation((SizeType)OldSize, (SizeType)NewSize, 1);
				}

				const uint32 NewTypeShift = UE::Core::CompactHashTable::GetTypeShift(NewMaxElements);

				// This should always be true since our type size will change on Powerof2 barriers (256, 65536, etc)
				check(NewTypeShift == UE::Core::CompactHashTable::GetTypeShift(MaxElements));

				const uint8* OldHashTable = Elements.GetAllocation() + GetElementsSizeInBytes(MaxElements, Layout);
				uint8* NewHashTable = Elements.GetAllocation() + GetElementsSizeInBytes(NewMaxElements, Layout);

				const uint8* OldHashLocation = OldHashTable + ((size_t)MaxElements << NewTypeShift) * KeyStorageMultiplier + HashCountSize;
				uint8* NewHashLocation = NewHashTable + ((size_t)NewMaxElements << NewTypeShift) * KeyStorageMultiplier + HashCountSize;

				// WARNING: Order matters here since we're moving data around in the same buffer. Make sure we're moving the tail data first
				// since we're expanding in size. So HashTable, then (optional) key storage, then hashcount and next indexes

				// Copy hash lookup table (first so to free up the space for the other data to move)
				FMemory::Memmove(NewHashLocation, OldHashLocation, ((size_t)NewHashCount << NewTypeShift));

#if UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE
				// Copy key storage if enabled
				const uint8* OldKeyStorage = OldHashTable + ((size_t)MaxElements << NewTypeShift) + HashCountSize;
				uint8* NewKeyStorage = NewHashTable + ((size_t)NewMaxElements << NewTypeShift) + HashCountSize;
				FMemory::Memmove(NewKeyStorage, OldKeyStorage, ((size_t)NumElements << NewTypeShift));
#endif

				// Copy hash size + next index table
				FMemory::Memmove(NewHashTable, OldHashTable, ((size_t)NumElements << NewTypeShift) + HashCountSize);
			}
			else
			{
				// If not preserving (or shrinking) then we only need to copy the array data
				if constexpr (TAllocatorTraits<AllocatorType>::SupportsElementAlignment)
				{
					Elements.ResizeAllocation(NumElements * Layout.Size, (SizeType)NewSize, 1, Layout.Alignment);
				}
				else
				{
					Elements.ResizeAllocation(NumElements * Layout.Size, (SizeType)NewSize, 1);
				}

				// Update hash size
				if (NewMaxElements > 0)
				{
					HashCountType* NewHashTable = (HashCountType*)(Elements.GetAllocation() + GetElementsSizeInBytes(NewMaxElements, Layout));

					*NewHashTable = NewHashCount;
					bRequiresRehash = true;
				}
			}

			MaxElements = NewMaxElements;
		}

		return bRequiresRehash;
	}

	ElementAllocatorType Elements;

	SizeType NumElements = 0;
	SizeType MaxElements = 0;
};
