// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/UnrealTemplate.h"
#include <atomic>
#include <type_traits>

namespace UE
{
	enum class EConsumeAllMpmcQueueResult
	{
		HadItems,
		WasEmpty,
	};

	/**
	 * Multi-producer/multi-consumer unbounded concurrent queue (implemented as a Stack) that is atomically consumed
	 * and is reset to its default empty state. Validated and run though atomic race detector.
	 */
	template<typename T, typename AllocatorType = FMemory>
	class TConsumeAllMpmcQueue final
	{
	private:
		struct FNode
		{
			std::atomic<FNode*> Next{ nullptr };
			TTypeCompatibleBytes<T> Item;
		};

		std::atomic<FNode*> Head{ nullptr };

	public:
		UE_NONCOPYABLE(TConsumeAllMpmcQueue);

		[[nodiscard]] TConsumeAllMpmcQueue() = default;

		~TConsumeAllMpmcQueue()
		{
			static_assert(std::is_trivially_destructible_v<FNode>);
			if (Head.load(std::memory_order_acquire) != nullptr)
			{
				ConsumeAllLifo([](T&&) {});
			}
		}

		//Push an Item to the Queue and 
		//returns EConsumeAllMpmcQueueResult::WasEmpty if the Queue was empty before
		//or EConsumeAllMpmcQueueResult::HadItems if there were already items in it
		template <typename... ArgTypes>
		EConsumeAllMpmcQueueResult ProduceItem(ArgTypes&&... Args)
		{
			FNode* New = ::new(AllocatorType::Malloc(sizeof(FNode), alignof(FNode))) FNode;
			::new ((void*)&New->Item) T(Forward<ArgTypes>(Args)...);

			//Atomically append to the top of the Queue
			FNode* Prev = Head.load(std::memory_order_relaxed);
			do
			{
				New->Next.store(Prev, std::memory_order_relaxed);
			} while (!Head.compare_exchange_weak(Prev, New, std::memory_order_acq_rel, std::memory_order_relaxed));
			
			return Prev == nullptr ? EConsumeAllMpmcQueueResult::WasEmpty : EConsumeAllMpmcQueueResult::HadItems;
		}

		//Take all items off the Queue atomically and consumes them in LIFO order
		//returns EConsumeAllMpmcQueueResult::WasEmpty if the Queue was empty
		//or EConsumeAllMpmcQueueResult::HadItems if there were items to consume
		template<typename F>
		EConsumeAllMpmcQueueResult ConsumeAllLifo(const F& Consumer)
		{
			return ConsumeAll<false>(Consumer);
		}

		//Take all items off the Queue atomically and consumes them in FIFO order
		//at the cost of the reversing the Links once
		//returns EConsumeAllMpmcQueueResult::WasEmpty if the Queue was empty
		//or EConsumeAllMpmcQueueResult::HadItems if there were items to consume
		template<typename F>
		EConsumeAllMpmcQueueResult ConsumeAllFifo(const F& Consumer)
		{
			return ConsumeAll<true>(Consumer);
		}

		// the result can be relied upon only in special cases (e.g. debug checks), as the state can change concurrently. use with caution 
		[[nodiscard]] bool IsEmpty() const
		{
			return Head.load(std::memory_order_relaxed) == nullptr;
		}

	private:
		template<bool bReverse, typename F>
		inline EConsumeAllMpmcQueueResult ConsumeAll(const F& Consumer)
		{
			//pop the entire Stack
			FNode* Node = Head.exchange(nullptr, std::memory_order_acq_rel);

			if (Node == nullptr)
			{
				return EConsumeAllMpmcQueueResult::WasEmpty;
			}

			if (bReverse) //reverse the links to FIFO Order if requested
			{
				FNode* Prev = nullptr;
				while (Node)
				{
					FNode* Tmp = Node;
					Node = Node->Next.exchange(Prev, std::memory_order_relaxed);
					Prev = Tmp;
				}
				Node = Prev;
			}

			while (Node) //consume the nodes of the Queue
			{
				FNode* Next = Node->Next.load(std::memory_order_relaxed);
				T* ValuePtr = Node->Item.GetTypedPtr();
				Consumer(MoveTemp(*ValuePtr));
				DestructItem(ValuePtr);
				AllocatorType::Free(Node);
				Node = Next;
			}

			return EConsumeAllMpmcQueueResult::HadItems;
		}
	};
}
