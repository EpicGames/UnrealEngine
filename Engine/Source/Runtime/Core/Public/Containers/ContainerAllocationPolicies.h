// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainerHelpers.h"
#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreMiscDefines.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/IsPolymorphic.h"
#include "Templates/MemoryOps.h"
#include "Templates/TypeCompatibleBytes.h"
#include <type_traits>


// Array slack tracking is a debug feature to track unused space in heap allocated TArray (and TScriptArray) structures.  This feature increases heap
// memory usage, and costs perf, so it is usually disabled by default.  Only works in builds where LLM is compiled in, but doesn't require -llm to be
// active.  Note that cooks will run quite slow with tracking enabled due to involving significantly more allocations than the engine, so be careful
// about leaving this enabled when kicking off a cook (including !WITH_EDITOR ensures it will only be enabled for client builds).  For more details,
// see additional comments in LowLevelMemTracker.cpp.
#ifndef UE_ENABLE_ARRAY_SLACK_TRACKING
#define UE_ENABLE_ARRAY_SLACK_TRACKING (0 && !WITH_EDITOR)
#endif


#if UE_ENABLE_ARRAY_SLACK_TRACKING

CORE_API uint8 LlmGetActiveTag();
CORE_API void ArraySlackTrackInit();
CORE_API void ArraySlackTrackGenerateReport(const TCHAR* Cmd, FOutputDevice& Ar);

// For detailed tracking of array slack waste, we need to add a header to heap allocations.  It's impossible to keep track of the TArray structure itself,
// because it can be inside other structures, and potentially moved around in unsafe ways, while the heap allocation the TArray points to is invariant.
// The heap allocations also have an advantage in that the contents are copied on Move or Realloc, simplifying tracking.
struct FArraySlackTrackingHeader
{
	FArraySlackTrackingHeader* Next;	// Linked list of tracked items
	FArraySlackTrackingHeader** Prev;
	uint16 AllocOffset;					// Offset below the header to the start of the actual allocation, to account for alignment padding
	uint8 Tag;
	int8 NumStackFrames;
	uint32 FirstAllocFrame;				// Frame where array allocation first occurred
	uint32 ReallocCount;				// Number of times realloc happened
	uint32 ArrayPeak;					// Peak observed ArrayNum
	uint64 ElemSize;

	// Note that we initially set the slack tracking ArrayNum to INDEX_NONE.  The container allocator is used by both arrays and
	// other containers (Set / Map / Hash), and we don't know it's actually an array until "UpdateNumUsed" is called on it.
	int64 ArrayNum;
	int64 ArrayMax;
	uint64 StackFrames[9];

	CORE_API void AddAllocation();
	CORE_API void RemoveAllocation();
	CORE_API void UpdateNumUsed(int64 NewNumUsed);
	CORE_API FORCENOINLINE static void* Realloc(void* Ptr, int64 Count, uint64 ElemSize, int32 Alignment);

	static void Free(void* Ptr)
	{
		if (Ptr)
		{
			FArraySlackTrackingHeader* TrackingHeader = (FArraySlackTrackingHeader*)((uint8*)Ptr - sizeof(FArraySlackTrackingHeader));
			TrackingHeader->RemoveAllocation();

			Ptr = (uint8*)TrackingHeader - TrackingHeader->AllocOffset;

			FMemory::Free(Ptr);
		}
	}

	static UE_FORCEINLINE_HINT void UpdateNumUsed(void* Ptr, int64 NewNumUsed)
	{
		if (Ptr)
		{
			FArraySlackTrackingHeader* TrackingHeader = (FArraySlackTrackingHeader*)((uint8*)Ptr - sizeof(FArraySlackTrackingHeader));

			TrackingHeader->UpdateNumUsed(NewNumUsed);
		}
	}

	static UE_FORCEINLINE_HINT void DisableTracking(void* Ptr)
	{
		if (Ptr)
		{
			FArraySlackTrackingHeader* TrackingHeader = (FArraySlackTrackingHeader*)((uint8*)Ptr - sizeof(FArraySlackTrackingHeader));

			TrackingHeader->RemoveAllocation();

			// When disabling tracking, we need to also reset ArrayNum, as it's used internally as a flag specifying whether
			// the allocation is currently tracked.  We don't reset this inside RemoveAllocation, because ArrayNum needs to
			// persist during realloc of tracked allocations, where RemoveAllocation is called, followed by AddAllocation.
			TrackingHeader->ArrayNum = INDEX_NONE;
		}
	}

	UE_FORCEINLINE_HINT int64 SlackSizeInBytes() const
	{
		return (ArrayMax - ArrayNum) * ElemSize;
	}
};
#endif  // UE_ENABLE_ARRAY_SLACK_TRACKING

// This option disables array slack for initial allocations, e.g where TArray::SetNum
// is called. This tends to save a lot of memory with almost no measured performance cost.
// NOTE: This can cause latent memory corruption issues to become more prominent
#ifndef CONTAINER_INITIAL_ALLOC_ZERO_SLACK
#define CONTAINER_INITIAL_ALLOC_ZERO_SLACK 1 // ON
#endif

#if defined(UE_CONTAINER_SLACK_GROWTH_FACTOR_NUMERATOR) && !defined(UE_CONTAINER_SLACK_GROWTH_FACTOR_DENOMINATOR)
	#error If UE_CONTAINER_SLACK_GROWTH_FACTOR_NUMERATOR is defined you must also define UE_CONTAINER_SLACK_GROWTH_FACTOR_DENOMINATOR
#endif

#if defined(UE_CONTAINER_SLACK_GROWTH_FACTOR_DENOMINATOR) && !defined(UE_CONTAINER_SLACK_GROWTH_FACTOR_NUMERATOR)
	#error If UE_CONTAINER_SLACK_GROWTH_FACTOR_DENOMINATOR is defined you must also define UE_CONTAINER_SLACK_GROWTH_FACTOR_NUMERATOR
#endif

#ifndef UE_CONTAINER_SLACK_GROWTH_FACTOR_NUMERATOR
	#if AGGRESSIVE_MEMORY_SAVING
		#define UE_CONTAINER_SLACK_GROWTH_FACTOR_NUMERATOR 1
	#else
		#define UE_CONTAINER_SLACK_GROWTH_FACTOR_NUMERATOR 3
	#endif
#endif

#ifndef UE_CONTAINER_SLACK_GROWTH_FACTOR_DENOMINATOR
	#if AGGRESSIVE_MEMORY_SAVING
		#define UE_CONTAINER_SLACK_GROWTH_FACTOR_DENOMINATOR 4
	#else
		#define UE_CONTAINER_SLACK_GROWTH_FACTOR_DENOMINATOR 8
	#endif
#endif
static_assert(UE_CONTAINER_SLACK_GROWTH_FACTOR_DENOMINATOR > 0, "UE_CONTAINER_SLACK_GROWTH_FACTOR_DENOMINATOR must be greater than 0");
static_assert(UE_CONTAINER_SLACK_GROWTH_FACTOR_DENOMINATOR > UE_CONTAINER_SLACK_GROWTH_FACTOR_NUMERATOR, "UE_CONTAINER_SLACK_GROWTH_FACTOR_DENOMINATOR must be greater than UE_CONTAINER_SLACK_GROWTH_FACTOR_NUMERATOR");

template <typename SizeType>
UE_FORCEINLINE_HINT SizeType DefaultCalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T BytesPerElement, bool bAllowQuantize, uint32 Alignment = DEFAULT_ALIGNMENT)
{
	SizeType Retval;
	checkSlow(NewMax < CurrentMax);

	// If the container has too much slack, shrink it to exactly fit the number of elements.
	const SizeType CurrentSlackElements = CurrentMax - NewMax;
	const SIZE_T CurrentSlackBytes = (CurrentMax - NewMax)*BytesPerElement;
	const bool bTooManySlackBytes = CurrentSlackBytes >= 16384;
	const bool bTooManySlackElements = 3 * NewMax < 2 * CurrentMax;
	if ((bTooManySlackBytes || bTooManySlackElements) && (CurrentSlackElements > 64 || !NewMax)) //  hard coded 64 :-(
	{
		Retval = NewMax;
		if (Retval > 0)
		{
			if (bAllowQuantize)
			{
				Retval = (SizeType)(FMemory::QuantizeSize(Retval * BytesPerElement, Alignment) / BytesPerElement);
			}
		}
	}
	else
	{
		Retval = CurrentMax;
	}

	return Retval;
}

template <typename SizeType>
UE_FORCEINLINE_HINT SizeType DefaultCalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T BytesPerElement, bool bAllowQuantize, uint32 Alignment = DEFAULT_ALIGNMENT)
{
#if !defined(AGGRESSIVE_MEMORY_SAVING)
	#error "AGGRESSIVE_MEMORY_SAVING must be defined"
#endif
#if AGGRESSIVE_MEMORY_SAVING
	const SIZE_T FirstGrow = 1;
	const SIZE_T ConstantGrow = 0;
#else
	const SIZE_T FirstGrow = 4;
	const SIZE_T ConstantGrow = 16;
#endif

	SizeType Retval;
	checkSlow(NewMax > CurrentMax && NewMax > 0);

	SIZE_T Grow = FirstGrow; // this is the amount for the first alloc

#if CONTAINER_INITIAL_ALLOC_ZERO_SLACK
	if (CurrentMax)
	{
		// Allocate slack for the array proportional to its size.
		Grow = SIZE_T(NewMax) + UE_CONTAINER_SLACK_GROWTH_FACTOR_NUMERATOR * SIZE_T(NewMax) / UE_CONTAINER_SLACK_GROWTH_FACTOR_DENOMINATOR + ConstantGrow;
	}
	else if (SIZE_T(NewMax) > Grow)
	{
		Grow = SIZE_T(NewMax);
	}
#else
	if (CurrentMax || SIZE_T(NewMax) > Grow)
	{
		// Allocate slack for the array proportional to its size.
		Grow = SIZE_T(NewMax) + UE_CONTAINER_SLACK_GROWTH_FACTOR_NUMERATOR * SIZE_T(NewMax) / UE_CONTAINER_SLACK_GROWTH_FACTOR_DENOMINATOR + ConstantGrow;
	}
#endif

	if (bAllowQuantize)
	{
		Retval = (SizeType)(FMemory::QuantizeSize(Grow * BytesPerElement, Alignment) / BytesPerElement);
	}
	else
	{
		Retval = (SizeType)Grow;
	}
	// NumElements and MaxElements are stored in 32 bit signed integers so we must be careful not to overflow here.
	if (NewMax > Retval)
	{
		Retval = TNumericLimits<SizeType>::Max();
	}

	return Retval;
}

template <typename SizeType>
UE_FORCEINLINE_HINT SizeType DefaultCalculateSlackReserve(SizeType NewMax, SIZE_T BytesPerElement, bool bAllowQuantize, uint32 Alignment = DEFAULT_ALIGNMENT)
{
	SizeType Retval = NewMax;
	checkSlow(NewMax > 0);
	if (bAllowQuantize)
	{
		Retval = (SizeType)(FMemory::QuantizeSize(SIZE_T(Retval) * SIZE_T(BytesPerElement), Alignment) / BytesPerElement);
		// NumElements and MaxElements are stored in 32 bit signed integers so we must be careful not to overflow here.
		if (NewMax > Retval)
		{
			Retval = TNumericLimits<SizeType>::Max();
		}
	}

	return Retval;
}

/** A type which is used to represent a script type that is unknown at compile time. */
struct FScriptContainerElement
{
};

template <typename AllocatorType>
struct TAllocatorTraitsBase
{
	enum { IsZeroConstruct           = false };
	enum { SupportsFreezeMemoryImage = false };
	enum { SupportsElementAlignment  = false };
	enum { SupportsSlackTracking     = false };
};

template <typename AllocatorType>
struct TAllocatorTraits : TAllocatorTraitsBase<AllocatorType>
{
};

template <typename FromAllocatorType, typename ToAllocatorType>
struct TCanMoveBetweenAllocators
{
	enum { Value = false };
};

/** This is the allocation policy interface; it exists purely to document the policy's interface, and should not be used. */
class FContainerAllocatorInterface
{
public:
	/** The integral type to be used for element counts and indices used by the allocator and container - must be signed */
	using SizeType = int32;

	/** Determines whether the user of the allocator may use the ForAnyElementType inner class. */
	enum { NeedsElementType = true };

	/** Determines whether the user of the allocator should do range checks */
	enum { RequireRangeCheck = true };

	/**
	 * A class that receives both the explicit allocation policy template parameters specified by the user of the container,
	 * but also the implicit ElementType template parameter from the container type.
	 */
	template<typename ElementType>
	class ForElementType
	{
		/**
		 * Moves the state of another allocator into this one.
		 *
		 * Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		 * @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		 */
		void MoveToEmpty(ForElementType& Other);

		/**
		 * Moves the state of another allocator into this one.  The allocator can be different, and the type must be specified.
		 * This function should only be called if TAllocatorTraits<AllocatorType>::SupportsMoveFromOtherAllocator is true.
		 *
		 * Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		 * @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		 */
		template <typename OtherAllocatorType>
		void MoveToEmptyFromOtherAllocator(typename OtherAllocatorType::template ForElementType<ElementType>& Other);

		/** Accesses the container's current data. */
		ElementType* GetAllocation() const;

		/**
		 * Resizes the container's allocation.
		 * @param CurrentNum - The number of elements that are currently constructed at the front of the allocation.
		 * @param NewMax - The number of elements to allocate space for.
		 * @param NumBytesPerElement - The number of bytes/element.
		 */
		void ResizeAllocation(
			SizeType CurrentNum,
			SizeType NewMax,
			SIZE_T NumBytesPerElement
		);

		/**
		 * Resizes the container's allocation.
		 * @param CurrentNum - The number of elements that are currently constructed at the front of the allocation.
		 * @param NewMax - The number of elements to allocate space for.
		 * @param NumBytesPerElement - The number of bytes/element.
		 * @param AlignmentOfElement - The alignment of the element type.
		 *
		 * @note  This overload only exists if TAllocatorTraits<Allocator>::SupportsElementAlignment == true.
		 */
		void ResizeAllocation(
			SizeType CurrentNum,
			SizeType NewMax,
			SIZE_T NumBytesPerElement,
			uint32 AlignmentOfElement
		);

		/**
		 * Calculates the amount of slack to allocate for an array that has just grown or shrunk to a given number of elements.
		 * @param NewMax - The number of elements to allocate space for.
		 * @param NumBytesPerElement - The number of bytes/element.
		 */
		SizeType CalculateSlackReserve(
			SizeType NewMax,
			SIZE_T NumBytesPerElement
		) const;

		/**
		 * Calculates the amount of slack to allocate for an array that has just grown or shrunk to a given number of elements.
		 * @param NewMax - The number of elements to allocate space for.
		 * @param NumBytesPerElement - The number of bytes/element.
		 * @param AlignmentOfElement - The alignment of the element type.
		 *
		 * @note  This overload only exists if TAllocatorTraits<Allocator>::SupportsElementAlignment == true.
		 */
		SizeType CalculateSlackReserve(
			SizeType NewMax,
			SIZE_T NumBytesPerElement,
			uint32 AlignmentOfElement
		) const;

		/**
		 * Calculates the amount of slack to allocate for an array that has just shrunk to a given number of elements.
		 * @param NewMax - The number of elements to allocate space for.
		 * @param CurrentMax - The number of elements for which space is currently allocated.
		 * @param NumBytesPerElement - The number of bytes/element.
		 */
		SizeType CalculateSlackShrink(
			SizeType NewMax,
			SizeType CurrentMax,
			SIZE_T NumBytesPerElement
			) const;

		/**
		 * Calculates the amount of slack to allocate for an array that has just shrunk to a given number of elements.
		 * @param NewMax - The number of elements to allocate space for.
		 * @param CurrentMax - The number of elements for which space is currently allocated.
		 * @param NumBytesPerElement - The number of bytes/element.
		 * @param AlignmentOfElement - The alignment of the element type.
		 *
		 * @note  This overload only exists if TAllocatorTraits<Allocator>::SupportsElementAlignment == true.
		 */
		SizeType CalculateSlackShrink(
			SizeType NewMax,
			SizeType CurrentMax,
			SIZE_T NumBytesPerElement,
			uint32 AlignmentOfElement
		) const;

		/**
		 * Calculates the amount of slack to allocate for an array that has just grown to a given number of elements.
		 * @param NewMax - The number of elements to allocate space for.
		 * @param CurrentMax - The number of elements for which space is currently allocated.
		 * @param NumBytesPerElement - The number of bytes/element.
		 */
		SizeType CalculateSlackGrow(
			SizeType NewMax,
			SizeType CurrentMax,
			SIZE_T NumBytesPerElement
		) const;

		/**
		 * Calculates the amount of slack to allocate for an array that has just grown to a given number of elements.
		 * @param NewMax - The number of elements to allocate space for.
		 * @param CurrentMax - The number of elements for which space is currently allocated.
		 * @param NumBytesPerElement - The number of bytes/element.
		 * @param AlignmentOfElement - The alignment of the element type.
		 *
		 * @note  This overload only exists if TAllocatorTraits<Allocator>::SupportsElementAlignment == true.
		 */
		SizeType CalculateSlackGrow(
			SizeType NewMax,
			SizeType CurrentMax,
			SIZE_T NumBytesPerElement,
			uint32 AlignmentOfElement
		) const;

		/**
		 * Returns the size of any requested heap allocation currently owned by the allocator.
		 * @param CurrentMax - The number of elements for which space is currently allocated.
		 * @param NumBytesPerElement - The number of bytes/element.
		 */
		SIZE_T GetAllocatedSize(SizeType CurrentMax, SIZE_T NumBytesPerElement) const;

		/** Returns true if the allocator has made any heap allocations */
		bool HasAllocation() const;

		/** Returns number of pre-allocated elements the container can use before allocating more space */
		SizeType GetInitialCapacity() const;

		/** Function called when ArrayNum changes for a TArray or TScriptArray, if TAllocatorTraits<Allocator>::SupportsSlackTracking == true */
		void SlackTrackerLogNum(SizeType NewNumUsed);
	};

	/**
	 * A class that may be used when NeedsElementType=false is specified.
	 * If NeedsElementType=true, then this must be present but will not be used, and so can simply be a typedef to void
	 */
	typedef ForElementType<FScriptContainerElement> ForAnyElementType;
};

namespace UE::Core::Private
{
	[[noreturn]] CORE_API void OnInvalidAlignedHeapAllocatorNum(int32 NewNum, SIZE_T NumBytesPerElement);
	[[noreturn]] CORE_API void OnInvalidSizedHeapAllocatorNum(int32 IndexSize, int64 NewNum, SIZE_T NumBytesPerElement);
}

/** A TArray allocator (32-bit sizetype only) that allocates from FMemory and allows specifying alignment for all allocations. */
template<uint32 Alignment = DEFAULT_ALIGNMENT>
class TAlignedHeapAllocator
{
public:
	using SizeType = int32;

	enum { NeedsElementType = false };
	enum { RequireRangeCheck = true };

	class ForAnyElementType
	{
	public:

		/** Default constructor. */
		ForAnyElementType()
			: Data(nullptr)
		{}

		/**
		 * Moves the state of another allocator into this one.
		 * Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		 * @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		 */
		UE_FORCEINLINE_HINT void MoveToEmpty(ForAnyElementType& Other)
		{
			checkSlow(this != &Other);

			if (Data)
			{
#if UE_ENABLE_ARRAY_SLACK_TRACKING
				FArraySlackTrackingHeader::Free(Data);
#else
				FMemory::Free(Data);
#endif
			}

			Data       = Other.Data;
			Other.Data = nullptr;
		}

		/** Destructor. */
		UE_FORCEINLINE_HINT ~ForAnyElementType()
		{
			if(Data)
			{
#if UE_ENABLE_ARRAY_SLACK_TRACKING
				FArraySlackTrackingHeader::Free(Data);
#else
				FMemory::Free(Data);
#endif
			}
		}

		// FContainerAllocatorInterface
		UE_FORCEINLINE_HINT FScriptContainerElement* GetAllocation() const
		{
			return Data;
		}
		void ResizeAllocation(
			SizeType CurrentNum,
			SizeType NewMax,
			SIZE_T NumBytesPerElement
			)
		{
			// Avoid calling FMemory::Realloc( nullptr, 0 ) as ANSI C mandates returning a valid pointer which is not what we want.
			if (Data || NewMax)
			{
				static_assert(sizeof(int32) <= sizeof(SIZE_T), "SIZE_T is expected to be larger than int32");

				// Check for under/overflow
				if (UNLIKELY(NewMax < 0 || NumBytesPerElement < 1 || NumBytesPerElement > (SIZE_T)MAX_int32))
				{
					UE::Core::Private::OnInvalidAlignedHeapAllocatorNum(NewMax, NumBytesPerElement);
				}

#if UE_ENABLE_ARRAY_SLACK_TRACKING
				Data = (FScriptContainerElement*)FArraySlackTrackingHeader::Realloc(Data, NewMax, NumBytesPerElement, Alignment > alignof(FArraySlackTrackingHeader) ? Alignment : alignof(FArraySlackTrackingHeader));
#else
				Data = (FScriptContainerElement*)FMemory::Realloc( Data, NewMax*NumBytesPerElement, Alignment );
#endif
			}
		}
		UE_FORCEINLINE_HINT SizeType CalculateSlackReserve(SizeType NewMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackReserve(NewMax, NumBytesPerElement, true, Alignment);
		}
		UE_FORCEINLINE_HINT SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, true, Alignment);
		}
		UE_FORCEINLINE_HINT SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackGrow(NewMax, CurrentMax, NumBytesPerElement, true, Alignment);
		}

		SIZE_T GetAllocatedSize(SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return CurrentMax * NumBytesPerElement;
		}

		bool HasAllocation() const
		{
			return !!Data;
		}

		SizeType GetInitialCapacity() const
		{
			return 0;
		}

#if UE_ENABLE_ARRAY_SLACK_TRACKING
		UE_FORCEINLINE_HINT void SlackTrackerLogNum(SizeType NewNumUsed)
		{
			FArraySlackTrackingHeader::UpdateNumUsed(Data, (int64)NewNumUsed);
		}

		// Suppress slack tracking on an allocation -- should be called whenever the container may have been resized.
		// Useful for debug allocations you don't want to show up in slack reports.
		UE_FORCEINLINE_HINT void DisableSlackTracking()
		{
			FArraySlackTrackingHeader::DisableTracking(Data);
		}
#endif

	private:
		ForAnyElementType(const ForAnyElementType&);
		ForAnyElementType& operator=(const ForAnyElementType&);

		/** A pointer to the container's elements. */
		FScriptContainerElement* Data;
	};

	template<typename ElementType>
	class ForElementType : public ForAnyElementType
	{
		static constexpr SIZE_T MinimumAlignment = (Alignment <= __STDCPP_DEFAULT_NEW_ALIGNMENT__) ? __STDCPP_DEFAULT_NEW_ALIGNMENT__ : Alignment;

	public:
		/** Default constructor. */
		ForElementType()
		{
			static_assert(alignof(ElementType) <= MinimumAlignment, "Using TAlignedHeapAllocator with an alignment lower than the element type's alignment - please update the alignment parameter");
		}

		UE_FORCEINLINE_HINT ElementType* GetAllocation() const
		{
			return (ElementType*)ForAnyElementType::GetAllocation();
		}
	};
};

template <uint32 Alignment>
struct TAllocatorTraits<TAlignedHeapAllocator<Alignment>> : TAllocatorTraitsBase<TAlignedHeapAllocator<Alignment>>
{
	enum { IsZeroConstruct = true };
	enum { SupportsSlackTracking = true };
};

template <int IndexSize>
struct TBitsToSizeType
{
	// Fabricate a compile-time false result that's still dependent on the template parameter
	static_assert(IndexSize == IndexSize+1, "Unsupported allocator index size.");
};

template <> struct TBitsToSizeType<8>  { using Type = int8; };
template <> struct TBitsToSizeType<16> { using Type = int16; };
template <> struct TBitsToSizeType<32> { using Type = int32; };
template <> struct TBitsToSizeType<64> { using Type = int64; };

/**
 * BaseClass for TArray Allocators.
 * Templated to support the container's sizetype (e.g. int32 (IndexSize==32) or int64 (IndexSize==64)).
 * Uses non-virtual inheritance via the CuriouslyRecurringTemplatePattern; concrete subclasses must pass
 * their own type in as the SubClass template parameter.
 */
template <int IndexSize, typename SubClass>
class TSizedAllocatorBase
{
public:
	using SizeType = typename TBitsToSizeType<IndexSize>::Type;

protected:
	using USizeType = std::make_unsigned_t<SizeType>;

public:
	enum { NeedsElementType = false };
	enum { RequireRangeCheck = true };

	class ForAnyElementType
	{
		template <int, typename>
		friend class TSizedAllocatorBase;

	public:
		/** Default constructor. */
		UE_NODEBUG constexpr ForAnyElementType()
			: Data(nullptr)
		{
		}

		/** Explicitly consteval constructor. */
		UE_NODEBUG explicit consteval ForAnyElementType(EConstEval)
			: Data(nullptr)
		{}

		/**
		 * Moves the state of another allocator into this one.  The allocator can be different.
		 *
		 * Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		 * @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		 */
		template <typename OtherAllocator>
		UE_NODEBUG UE_FORCEINLINE_HINT void MoveToEmptyFromOtherAllocator(typename OtherAllocator::ForAnyElementType& Other)
		{
			checkSlow((void*)this != (void*)&Other);

			if (Data)
			{
				// Call subclass Free using CuriouslyRecurringTemplatePattern.
				static_cast<SubClass::ForAnyElementType*>(this)->CallFree(Data);
			}

			Data = Other.Data;
			Other.Data = nullptr;
		}

		/**
		 * Moves the state of another allocator into this one. The allocator is the same class.
		 *
		 * Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		 * @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		 */
		UE_NODEBUG UE_FORCEINLINE_HINT void MoveToEmpty(SubClass::ForAnyElementType& Other)
		{
			this->MoveToEmptyFromOtherAllocator<SubClass>(Other);
		}

		/** Destructor. */
		UE_NODEBUG UE_FORCEINLINE_HINT ~ForAnyElementType()
		{
			if (Data)
			{
				// Call subclass Free using CuriouslyRecurringTemplatePattern.
				static_cast<SubClass::ForAnyElementType*>(this)->CallFree(Data);
			}
		}

		// FContainerAllocatorInterface
		UE_NODEBUG UE_FORCEINLINE_HINT FScriptContainerElement* GetAllocation() const
		{
			return Data;
		}
		UE_NODEBUG void ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement)
		{
			// Avoid calling FMemory::Realloc( nullptr, 0 ) as ANSI C mandates returning a valid pointer which is not what we want.
			if (Data || NewMax)
			{
				static_assert(sizeof(SizeType) <= sizeof(SIZE_T), "SIZE_T is expected to handle all possible sizes");

				// Check for under/overflow
				bool bInvalidResize = NewMax < 0 || NumBytesPerElement < 1 || NumBytesPerElement > (SIZE_T)MAX_int32;
				if constexpr (sizeof(SizeType) == sizeof(SIZE_T))
				{
					bInvalidResize = bInvalidResize || (SIZE_T)(USizeType)NewMax > (SIZE_T)TNumericLimits<SizeType>::Max() / NumBytesPerElement;
				}
				if (UNLIKELY(bInvalidResize))
				{
					UE::Core::Private::OnInvalidSizedHeapAllocatorNum(IndexSize, NewMax, NumBytesPerElement);
				}

				// Call subclass Realloc using CuriouslyRecurringTemplatePattern.
				Data = (FScriptContainerElement*)static_cast<SubClass::ForAnyElementType*>(this)->CallRealloc(Data, NewMax, NumBytesPerElement);
			}
		}
		UE_NODEBUG void ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement)
		{
			// Avoid calling Realloc( nullptr, 0 ) as ANSI C mandates returning a valid pointer which is not what we want.
			if (Data || NewMax)
			{
				static_assert(sizeof(SizeType) <= sizeof(SIZE_T), "SIZE_T is expected to handle all possible sizes");

				// Check for under/overflow
				bool bInvalidResize = NewMax < 0 || NumBytesPerElement < 1 || NumBytesPerElement > (SIZE_T)MAX_int32;
				if constexpr (sizeof(SizeType) == sizeof(SIZE_T))
				{
					bInvalidResize = bInvalidResize || ((SIZE_T)(USizeType)NewMax > (SIZE_T)TNumericLimits<SizeType>::Max() / NumBytesPerElement);
				}
				if (UNLIKELY(bInvalidResize))
				{
					UE::Core::Private::OnInvalidSizedHeapAllocatorNum(IndexSize, NewMax, NumBytesPerElement);
				}

				// Call subclass Realloc using CuriouslyRecurringTemplatePattern.
				Data = (FScriptContainerElement*)static_cast<SubClass::ForAnyElementType*>(this)->CallRealloc(Data, NewMax, NumBytesPerElement, AlignmentOfElement);
			}
		}
		UE_NODEBUG UE_FORCEINLINE_HINT SizeType CalculateSlackReserve(SizeType NewMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackReserve(NewMax, NumBytesPerElement, true);
		}
		UE_NODEBUG UE_FORCEINLINE_HINT SizeType CalculateSlackReserve(SizeType NewMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement) const
		{
			return DefaultCalculateSlackReserve(NewMax, NumBytesPerElement, true, (uint32)AlignmentOfElement);
		}
		UE_NODEBUG UE_FORCEINLINE_HINT SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, true);
		}
		UE_NODEBUG UE_FORCEINLINE_HINT SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement) const
		{
			return DefaultCalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, true, (uint32)AlignmentOfElement);
		}
		UE_NODEBUG UE_FORCEINLINE_HINT SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackGrow(NewMax, CurrentMax, NumBytesPerElement, true);
		}
		UE_NODEBUG UE_FORCEINLINE_HINT SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement) const
		{
			return DefaultCalculateSlackGrow(NewMax, CurrentMax, NumBytesPerElement, true, (uint32)AlignmentOfElement);
		}

		UE_NODEBUG SIZE_T GetAllocatedSize(SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return CurrentMax * NumBytesPerElement;
		}

		UE_NODEBUG bool HasAllocation() const
		{
			return !!Data;
		}

		UE_NODEBUG constexpr SizeType GetInitialCapacity() const
		{
			return 0;
		}

	public:
		// Optional template functions to override on ForAnyElementType on the SubClass.
#if UE_ENABLE_ARRAY_SLACK_TRACKING
		UE_NODEBUG UE_FORCEINLINE_HINT void SlackTrackerLogNum(SizeType NewNumUsed)
		{
			// Base implementation does not report to SlackTracking; subclasses must override if they need to report
			// to slacktracking, @see TSizedHeapAllocator. 
		}

		// Suppress slack tracking on an allocation -- should be called whenever the container may have been resized.
		// Useful for debug allocations you don't want to show up in slack reports.
		UE_NODEBUG UE_FORCEINLINE_HINT void DisableSlackTracking()
		{
			// Base implementation does not report to SlackTracking; subclasses must override if they need to report
			// to slacktracking, @see TSizedHeapAllocator. 
		}
#endif

		// Required template functions to override on ForAnyElementType on the SubClass.
		void* CallRealloc(void* InData, SizeType NewMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement = DEFAULT_ALIGNMENT) = delete;
		void CallFree(void* InData) = delete;

	protected:
		ForAnyElementType(const ForAnyElementType&);
		ForAnyElementType& operator=(const ForAnyElementType&);

		/** A pointer to the container's elements. */
		FScriptContainerElement* Data;
	};

	template<typename ElementType>
	class ForElementType : public SubClass::ForAnyElementType
	{
	public:
		ForElementType() = default;
		UE_NODEBUG UE_FORCEINLINE_HINT ElementType* GetAllocation() const
		{
			return (ElementType*)ForAnyElementType::GetAllocation();
		}
	};
};

/**
 * A TArray allocator that allocates from FMemory.
 * The BaseMallocType template parameter is deprecated and passing it in will be a compile error in a future release.
 * Use TSizedExternalAllocator instead to provide an ExternalMallocType,
 */
template <int IndexSize, typename BaseMallocType = FMemory>
class TSizedHeapAllocator : public TSizedAllocatorBase<IndexSize, TSizedHeapAllocator<IndexSize, BaseMallocType>>
{
public:
	using MyClass = TSizedHeapAllocator<IndexSize, BaseMallocType>;
	using SuperClass = TSizedAllocatorBase<IndexSize, MyClass>;
	using BaseMalloc = BaseMallocType;
	using SizeType = SuperClass::SizeType;

	class ForAnyElementType : public SuperClass::ForAnyElementType
	{
	public:
		// Declare we are using constructors from ForAnyElementType on SuperClass
		using SuperClass::ForAnyElementType::ForAnyElementType;

		// Declare we are using Data pointer on the superclass; the compiler can't figure this out automatically due
		// to two-phase template compile.
		using SuperClass::ForAnyElementType::Data;

		// Declare our constructor so we can add deprecation handling for BaseMallocType.
		UE_NODEBUG constexpr ForAnyElementType()
		{
			UE_STATIC_DEPRECATE(5.8, (!std::is_same_v<BaseMallocType, FMemory>), "Overriding the MallocType in TSizedHeapAllocator is deprecated. Use TSizedExternalAllocator<int IndexSize, typename ExternalMallocType> instead");
		}

		// Override the ResizeAllocation functions so they can be defined as exported to avoid bloat
#if !UE_MERGED_MODULES && !defined(UE_HEADER_UNITS)
		UE_NODEBUG void ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement)
		{
			SuperClass::ForAnyElementType::ResizeAllocation(CurrentNum, NewMax, NumBytesPerElement);
		}
		UE_NODEBUG void ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement)
		{
			SuperClass::ForAnyElementType::ResizeAllocation(CurrentNum, NewMax, NumBytesPerElement, AlignmentOfElement);
		}
#endif

	public:
#if UE_ENABLE_ARRAY_SLACK_TRACKING
		UE_NODEBUG UE_FORCEINLINE_HINT void SlackTrackerLogNum(SizeType NewNumUsed)
		{
			FArraySlackTrackingHeader::UpdateNumUsed(Data, (int64)NewNumUsed);
		}

		UE_NODEBUG UE_FORCEINLINE_HINT void DisableSlackTracking()
		{
			FArraySlackTrackingHeader::DisableTracking(Data);
		}
#endif

		UE_NODEBUG UE_FORCEINLINE_HINT void* CallRealloc(void* InData, SuperClass::SizeType NewMax,
			SIZE_T NumBytesPerElement, uint32 AlignmentOfElement = DEFAULT_ALIGNMENT)
		{
#if UE_ENABLE_ARRAY_SLACK_TRACKING
			AlignmentOfElement = AlignmentOfElement == 0
				? 0
				: (AlignmentOfElement > alignof(FArraySlackTrackingHeader)
					? AlignmentOfElement : alignof(FArraySlackTrackingHeader));
			return (FScriptContainerElement*)FArraySlackTrackingHeader::Realloc(InData, NewMax, NumBytesPerElement, AlignmentOfElement);
#else
			return (FScriptContainerElement*)BaseMalloc::Realloc(InData, NewMax * NumBytesPerElement, AlignmentOfElement);
#endif
		}
		UE_NODEBUG UE_FORCEINLINE_HINT void CallFree(void* InData)
		{
#if UE_ENABLE_ARRAY_SLACK_TRACKING
			FArraySlackTrackingHeader::Free(InData);
#else
			BaseMalloc::Free(InData);
#endif
		}
	};

	// Special implementation of ForElementType that implements the consteval constructor.
	template<typename ElementType>
	class ForElementType : public ForAnyElementType
	{
	public:
		UE_NODEBUG ForElementType() = default;
		UE_NODEBUG explicit consteval ForElementType(EConstEval)
			: ForAnyElementType(ConstEval)
		{
		}
		UE_NODEBUG UE_FORCEINLINE_HINT ElementType* GetAllocation() const
		{
			return (ElementType*)ForAnyElementType::GetAllocation();
		}
	};
};

/** A SizedAllocator that allocates from a compile-time-provided set of allocation functions. */
template <int IndexSize, typename ExternalMallocType>
class TSizedExternalAllocator : public TSizedAllocatorBase<IndexSize, TSizedExternalAllocator<IndexSize, ExternalMallocType>>
{
public:
	using MyClass = TSizedExternalAllocator<IndexSize, ExternalMallocType>;
	using SuperClass = TSizedAllocatorBase<IndexSize, MyClass>;
	using ExternalMalloc = ExternalMallocType;

	class ForAnyElementType : public SuperClass::ForAnyElementType
	{
	public:
		// Declare we are using constructors from ForAnyElementType on SuperClass
		using SuperClass::ForAnyElementType::ForAnyElementType;

	public:
		UE_FORCEINLINE_HINT void* CallRealloc(void* InData, SuperClass::SizeType NewMax,
			SIZE_T NumBytesPerElement, uint32 AlignmentOfElement = DEFAULT_ALIGNMENT)
		{
			return (FScriptContainerElement*)ExternalMalloc::Realloc(InData, NewMax * NumBytesPerElement, AlignmentOfElement);
		}
		UE_FORCEINLINE_HINT void CallFree(void* InData)
		{
			ExternalMalloc::Free(InData);
		}
	};
};

/**
 * A SizedAllocator that allocates from a runtime-provided pointer to an FMalloc interface.
 * TArrays using this allocator must be constructed with the TArray constructors that take an ElementAllocatorType&&
 * argument; attempting to construct them without that argument will intentionally fail due to lack of a default constructor
 * on ForAnyElementType.
*/
template <int IndexSize>
class TSizedMallocAllocator : public TSizedAllocatorBase<IndexSize, TSizedMallocAllocator<IndexSize>>
{
public:
	using MyClass = TSizedMallocAllocator<IndexSize>;
	using SuperClass = TSizedAllocatorBase<IndexSize, MyClass>;

	class ForAnyElementType : public SuperClass::ForAnyElementType
	{
	public:
		// Declare we are using Data pointer on the superclass; the compiler can't figure this out automatically due
		// to two-phase template compile.
		using SuperClass::ForAnyElementType::Data;

		explicit ForAnyElementType(FMalloc* InMalloc)
			: Malloc(InMalloc)
		{
		}
		ForAnyElementType(ForAnyElementType&& Other)
			: Malloc(Other.Malloc)
		{
			Data = Other.Data;
			Other.Data = nullptr;
		}

	public:
		UE_FORCEINLINE_HINT void* CallRealloc(void* InData, SuperClass::SizeType NewMax,
			SIZE_T NumBytesPerElement, uint32 AlignmentOfElement = DEFAULT_ALIGNMENT)
		{
			return Malloc->Realloc(InData, NewMax * NumBytesPerElement, AlignmentOfElement);
		}
		UE_FORCEINLINE_HINT void CallFree(void* InData)
		{
			Malloc->Free(InData);
		}

	protected:
		FMalloc* Malloc;
	};

	// Declare our own ForElementType so we can support our constructor parameters.
	template<typename ElementType>
	class ForElementType : public ForAnyElementType
	{
	public:
		explicit ForElementType(FMalloc* InMalloc)
			: ForAnyElementType(InMalloc)
		{
		}
		ForElementType(ForElementType&& Other)
			: ForAnyElementType(MoveTemp(Other))
		{
		}

		UE_FORCEINLINE_HINT ElementType* GetAllocation() const
		{
			return (ElementType*)ForAnyElementType::GetAllocation();
		}
	};
};

template <int IndexSize> class TSizedDefaultAllocator : public TSizedHeapAllocator<IndexSize> { public: typedef TSizedHeapAllocator<IndexSize> Typedef; };
using FDefaultAllocator = TSizedDefaultAllocator<32>;

using FDefaultMallocAllocator = TSizedMallocAllocator<32>;

/** TSizedNonshrinkingAllocator behaves like TSizedDefaultAllocator, but prevents arrays from automatically shrinking unless EAllowShrinking::Yes is used. */
template <int IndexSize> class TSizedNonshrinkingAllocator : public TSizedHeapAllocator<IndexSize>
{
public:
	typedef TSizedHeapAllocator<IndexSize> Typedef;
	enum { ShrinkByDefault = false };
};

// Define the ResizeAllocation functions with the regular allocator as exported to avoid bloat
#if !UE_MERGED_MODULES && !defined(UE_HEADER_UNITS)
extern template CORE_API FORCENOINLINE void TSizedHeapAllocator<32, FMemory>::ForAnyElementType::ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement);
extern template CORE_API FORCENOINLINE void TSizedHeapAllocator<32, FMemory>::ForAnyElementType::ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement);
extern template CORE_API FORCENOINLINE void TSizedHeapAllocator<64, FMemory>::ForAnyElementType::ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement);
extern template CORE_API FORCENOINLINE void TSizedHeapAllocator<64, FMemory>::ForAnyElementType::ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement);
#endif

template <uint8 IndexSize>
struct TAllocatorTraits<TSizedHeapAllocator<IndexSize>> : TAllocatorTraitsBase<TSizedHeapAllocator<IndexSize>>
{
	enum { IsZeroConstruct          = true };
	enum { SupportsElementAlignment = true };
	enum { SupportsSlackTracking    = true };
};

using FHeapAllocator = TSizedHeapAllocator<32>;

template <uint8 FromIndexSize, uint8 ToIndexSize>
struct TCanMoveBetweenAllocators<TSizedHeapAllocator<FromIndexSize>, TSizedHeapAllocator<ToIndexSize>>
{
	// Allow conversions between different int width versions of the allocator
	enum { Value = true };
};

/**
 * The inline allocation policy allocates up to a specified number of elements in the same allocation as the container.
 * Any allocation needed beyond that causes all data to be moved into an indirect allocation.
 * It always uses DEFAULT_ALIGNMENT.
 */
namespace UE::Core::Private
{
	// This concept allows us to support allocators which don't specify `enum { ShrinkByDefault = ...; }`.
	template <typename AllocatorType>
	concept CHasShrinkByDefault = requires { AllocatorType::ShrinkByDefault; };

	// Returns the value of AllocatorType::ShrinkByDefault if the enum exists, or bFallback if the enum is absent.
	template <bool bFallback, typename AllocatorType>
	consteval bool ShrinkByDefaultOr()
	{
		if constexpr (UE::Core::Private::CHasShrinkByDefault<AllocatorType>)
		{
			return AllocatorType::ShrinkByDefault;
		}
		else
		{
			return bFallback;
		}
	}
}

template <uint32 NumInlineElements, int IndexSize, typename SecondaryAllocator = FDefaultAllocator>
class TSizedInlineAllocator
{
public:
	using SizeType = typename TBitsToSizeType<IndexSize>::Type;

	static_assert(std::is_same_v<SizeType, typename SecondaryAllocator::SizeType>, "Secondary allocator SizeType mismatch");

	enum { NeedsElementType = true };
	enum { RequireRangeCheck = true };
	enum { ShrinkByDefault = UE::Core::Private::ShrinkByDefaultOr<true, SecondaryAllocator>() };

	template<typename ElementType>
	class ForElementType
	{
	public:

		/** Default constructor. */
		constexpr ForElementType()
		{
			UE_IF_CONSTEVAL
			{
				for (uint8& Byte : InlineData)
				{
					Byte = 0;
				}
			}
		}

		explicit consteval ForElementType(EConstEval)
			: InlineData() // Force value initialization
			, SecondaryData(ConstEval)
		{
		}

		/**
		 * Moves the state of another allocator into this one.
		 * Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		 * @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		 */
		UE_FORCEINLINE_HINT void MoveToEmpty(ForElementType& Other)
		{
			checkSlow(this != &Other);

			if (!Other.SecondaryData.GetAllocation())
			{
				// Relocate objects from other inline storage only if it was stored inline in Other
				RelocateConstructItems<ElementType>((void*)InlineData, Other.GetInlineElements(), NumInlineElements);
			}

			// Move secondary storage in any case.
			// This will move secondary storage if it exists but will also handle the case where secondary storage is used in Other but not in *this.
			SecondaryData.MoveToEmpty(Other.SecondaryData);
		}

		// FContainerAllocatorInterface
		UE_NODEBUG UE_FORCEINLINE_HINT ElementType* GetAllocation() const
		{
			if (ElementType* Result = SecondaryData.GetAllocation())
			{
				return Result;
			}
			return GetInlineElements();
		}

		void ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement)
			requires (!TAllocatorTraits<SecondaryAllocator>::SupportsElementAlignment)
		{
			// Make sure the number of live elements is still withing the allocation since we only memmove, not destruct
			checkSlow(CurrentNum <= NewMax);

			// Check if the new allocation will fit in the inline data area.
			if (NewMax <= NumInlineElements)
			{
				// If the old allocation wasn't in the inline data area, relocate it into the inline data area.
				if (SecondaryData.GetAllocation())
				{
					RelocateConstructItems<ElementType>((void*)InlineData, (ElementType*)SecondaryData.GetAllocation(), CurrentNum);

					// Free the old indirect allocation.
					SecondaryData.ResizeAllocation(0, 0, NumBytesPerElement);
				}
			}
			else
			{
				if (!SecondaryData.GetAllocation())
				{
					// Allocate new indirect memory for the data.
					SecondaryData.ResizeAllocation(0, NewMax, NumBytesPerElement);

					// Move the data out of the inline data area into the new allocation.
					RelocateConstructItems<ElementType>((void*)SecondaryData.GetAllocation(), GetInlineElements(), CurrentNum);
				}
				else
				{
					// Reallocate the indirect data for the new size.
					SecondaryData.ResizeAllocation(CurrentNum, NewMax, NumBytesPerElement);
				}
			}
		}

		void ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement = DEFAULT_ALIGNMENT)
			requires (!!TAllocatorTraits<SecondaryAllocator>::SupportsElementAlignment)
		{
			// Make sure the number of live elements is still withing the allocation since we only memmove, not destruct
			checkSlow(CurrentNum <= NewMax);

			// Check if the new allocation will fit in the inline data area.
			if (NewMax <= NumInlineElements)
			{
				// If the old allocation wasn't in the inline data area, relocate it into the inline data area.
				if (SecondaryData.GetAllocation())
				{
					RelocateConstructItems<ElementType>((void*)InlineData, (ElementType*)SecondaryData.GetAllocation(), CurrentNum);

					// Free the old indirect allocation.
					SecondaryData.ResizeAllocation(0, 0, NumBytesPerElement, AlignmentOfElement);
				}
			}
			else
			{
				if (!SecondaryData.GetAllocation())
				{
					// Allocate new indirect memory for the data.
					SecondaryData.ResizeAllocation(0, NewMax, NumBytesPerElement, AlignmentOfElement);

					// Move the data out of the inline data area into the new allocation.
					RelocateConstructItems<ElementType>((void*)SecondaryData.GetAllocation(), GetInlineElements(), CurrentNum);
				}
				else
				{
					// Reallocate the indirect data for the new size.
					SecondaryData.ResizeAllocation(CurrentNum, NewMax, NumBytesPerElement, AlignmentOfElement);
				}
			}
		}

		UE_NODEBUG UE_FORCEINLINE_HINT SizeType CalculateSlackReserve(SizeType NewMax, SIZE_T NumBytesPerElement) const
			requires (!TAllocatorTraits<SecondaryAllocator>::SupportsElementAlignment)
		{
			// If the elements use less space than the inline allocation, only use the inline allocation as slack.
			return NewMax <= NumInlineElements ?
				NumInlineElements :
				SecondaryData.CalculateSlackReserve(NewMax, NumBytesPerElement);
		}
		UE_NODEBUG UE_FORCEINLINE_HINT SizeType CalculateSlackReserve(SizeType NewMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement = DEFAULT_ALIGNMENT) const
			requires (!!TAllocatorTraits<SecondaryAllocator>::SupportsElementAlignment)
		{
			// If the elements use less space than the inline allocation, only use the inline allocation as slack.
			return NewMax <= NumInlineElements ?
				NumInlineElements :
				SecondaryData.CalculateSlackReserve(NewMax, NumBytesPerElement, AlignmentOfElement);
		}
		UE_NODEBUG UE_FORCEINLINE_HINT SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
			requires (!TAllocatorTraits<SecondaryAllocator>::SupportsElementAlignment)
		{
			// If the elements use less space than the inline allocation, only use the inline allocation as slack.
			return NewMax <= NumInlineElements ?
				NumInlineElements :
				SecondaryData.CalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement);
		}
		UE_NODEBUG UE_FORCEINLINE_HINT SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement = DEFAULT_ALIGNMENT) const
			requires (!!TAllocatorTraits<SecondaryAllocator>::SupportsElementAlignment)
		{
			// If the elements use less space than the inline allocation, only use the inline allocation as slack.
			return NewMax <= NumInlineElements ?
				NumInlineElements :
				SecondaryData.CalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, AlignmentOfElement);
		}
		UE_NODEBUG UE_FORCEINLINE_HINT SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
			requires (!TAllocatorTraits<SecondaryAllocator>::SupportsElementAlignment)
		{
			// If the elements use less space than the inline allocation, only use the inline allocation as slack.
			// Also, when computing slack growth, don't count inline elements -- the slack algorithm has a special
			// case to save memory on the initial heap allocation, versus subsequent reallocations, and we don't
			// want the inline elements to be treated as if they were the first heap allocation.
			return NewMax <= NumInlineElements ?
				NumInlineElements :
				SecondaryData.CalculateSlackGrow(NewMax, CurrentMax <= NumInlineElements ? 0 : CurrentMax, NumBytesPerElement);
		}
		UE_NODEBUG UE_FORCEINLINE_HINT SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement = DEFAULT_ALIGNMENT) const
			requires (!!TAllocatorTraits<SecondaryAllocator>::SupportsElementAlignment)
		{
			// If the elements use less space than the inline allocation, only use the inline allocation as slack.
			// Also, when computing slack growth, don't count inline elements -- the slack algorithm has a special
			// case to save memory on the initial heap allocation, versus subsequent reallocations, and we don't
			// want the inline elements to be treated as if they were the first heap allocation.
			return NewMax <= NumInlineElements ?
				NumInlineElements :
				SecondaryData.CalculateSlackGrow(NewMax, CurrentMax <= NumInlineElements ? 0 : CurrentMax, NumBytesPerElement, AlignmentOfElement);
		}

		UE_NODEBUG UE_FORCEINLINE_HINT SIZE_T GetAllocatedSize(SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			if (CurrentMax > NumInlineElements)
			{
				return SecondaryData.GetAllocatedSize(CurrentMax, NumBytesPerElement);
			}
			return 0;
		}

		UE_REWRITE bool HasAllocation() const
		{
			return SecondaryData.HasAllocation();
		}

		constexpr SizeType GetInitialCapacity() const
		{
			return NumInlineElements;
		}

#if UE_ENABLE_ARRAY_SLACK_TRACKING
		UE_FORCEINLINE_HINT void SlackTrackerLogNum(SizeType NewNumUsed)
		{
			if constexpr (TAllocatorTraits<SecondaryAllocator>::SupportsSlackTracking)
			{
				SecondaryData.SlackTrackerLogNum(NewNumUsed);
			}
		}
#endif

	private:
		ForElementType(const ForElementType&);
		ForElementType& operator=(const ForElementType&);

		/** The data is stored in this array if less than NumInlineElements is needed. Uninitialized by default. */
		alignas(ElementType) uint8 InlineData[sizeof(ElementType) * NumInlineElements];

		/** The data is allocated through the indirect allocation policy if more than NumInlineElements is needed. */
		typename SecondaryAllocator::template ForElementType<ElementType> SecondaryData;

		/** @return the base of the aligned inline element data */
		UE_REWRITE ElementType* GetInlineElements() const
		{
			return (ElementType*)InlineData;
		}
	};

	typedef void ForAnyElementType;
};

template <uint32 NumInlineElements, int IndexSize, typename SecondaryAllocator>
struct TAllocatorTraits<TSizedInlineAllocator<NumInlineElements, IndexSize, SecondaryAllocator>> : TAllocatorTraitsBase<TSizedInlineAllocator<NumInlineElements, IndexSize, SecondaryAllocator>>
{
	enum { SupportsElementAlignment = TAllocatorTraits<SecondaryAllocator>::SupportsElementAlignment };
	enum { SupportsSlackTracking = true };
};

template <uint32 NumInlineElements, typename SecondaryAllocator = FDefaultAllocator>
using TInlineAllocator = TSizedInlineAllocator<NumInlineElements, 32, SecondaryAllocator>;

template <uint32 NumInlineElements, typename SecondaryAllocator = FDefaultAllocator64>
using TInlineAllocator64 = TSizedInlineAllocator<NumInlineElements, 64, SecondaryAllocator>;

/**
 * Implements a variant of TInlineAllocator with a secondary heap allocator that is allowed to store a pointer to its inline elements.
 * This allows caching a pointer to the elements which avoids any conditional logic in GetAllocation(), but prevents the allocator being trivially relocatable.
 * All UE allocators typically rely on elements being trivially relocatable, so instances of this allocator cannot be used in other containers.
 *
 * NOTE: instances of this allocator - or containers which use them - are non-trivially-relocatable, but the allocator still expects elements themselves to be trivially-relocatable.
 */
template <uint32 NumInlineElements>
class TNonRelocatableInlineAllocator
{
public:
	using SizeType = int32;

	enum { NeedsElementType = true };
	enum { RequireRangeCheck = true };

	template<typename ElementType>
	class ForElementType
	{
	public:
		/** Default constructor. */
		ForElementType()
			: Data(GetInlineElements())
		{
			static_assert(alignof(ElementType) <= __STDCPP_DEFAULT_NEW_ALIGNMENT__, "TNonRelocatableInlineAllocator uses GMalloc's default alignment, which is lower than the element type's alignment - please consider a different approach");
		}

		~ForElementType()
		{
			if (HasAllocation())
			{
#if UE_ENABLE_ARRAY_SLACK_TRACKING
				FArraySlackTrackingHeader::Free(Data);
#else
				FMemory::Free(Data);
#endif
			}
		}

		/**
		 * Moves the state of another allocator into this one.
		 * Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		 * @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		 */
		UE_FORCEINLINE_HINT void MoveToEmpty(ForElementType& Other)
		{
			checkSlow(this != &Other);

			if (HasAllocation())
			{
#if UE_ENABLE_ARRAY_SLACK_TRACKING
				FArraySlackTrackingHeader::Free(Data);
#else
				FMemory::Free(Data);
#endif
			}

			if (Other.HasAllocation())
			{
				Data = Other.Data;
				Other.Data = Other.GetInlineElements();
			}
			else
			{
				Data = GetInlineElements();
				RelocateConstructItems<ElementType>(GetInlineElements(), Other.GetInlineElements(), NumInlineElements);
			}
		}

		// FContainerAllocatorInterface
		UE_FORCEINLINE_HINT ElementType* GetAllocation() const
		{
			return Data;
		}

		void ResizeAllocation(SizeType CurrentNum, SizeType NewMax,SIZE_T NumBytesPerElement)
		{
			// Make sure the number of live elements is still withing the allocation since we only memmove, not destruct
			checkSlow(CurrentNum <= NewMax);

			// Check if the new allocation will fit in the inline data area.
			if(NewMax <= NumInlineElements)
			{
				// If the old allocation wasn't in the inline data area, relocate it into the inline data area.
				if(HasAllocation())
				{
					RelocateConstructItems<ElementType>(GetInlineElements(), Data, CurrentNum);
#if UE_ENABLE_ARRAY_SLACK_TRACKING
					FArraySlackTrackingHeader::Free(Data);
#else
					FMemory::Free(Data);
#endif
					Data = GetInlineElements();
				}
			}
			else
			{
				if (HasAllocation())
				{
					// Reallocate the indirect data for the new size.
#if UE_ENABLE_ARRAY_SLACK_TRACKING
					Data = (ElementType*)FArraySlackTrackingHeader::Realloc(Data, (int32)NewMax, (int32)NumBytesPerElement, 0);
#else
					Data = (ElementType*)FMemory::Realloc(Data, NewMax*NumBytesPerElement);
#endif
				}
				else
				{
					// Allocate new indirect memory for the data.
#if UE_ENABLE_ARRAY_SLACK_TRACKING
					Data = (ElementType*)FArraySlackTrackingHeader::Realloc(nullptr, (int32)NewMax, (int32)NumBytesPerElement, 0);
#else
					Data = (ElementType*)FMemory::Realloc(nullptr, NewMax*NumBytesPerElement);
#endif

					// Move the data out of the inline data area into the new allocation.
					RelocateConstructItems<ElementType>(Data, GetInlineElements(), CurrentNum);
				}
			}
		}

		UE_FORCEINLINE_HINT SizeType CalculateSlackReserve(SizeType NewMax, SIZE_T NumBytesPerElement) const
		{
			// If the elements use less space than the inline allocation, only use the inline allocation as slack.
			return (NewMax <= NumInlineElements) ? NumInlineElements : DefaultCalculateSlackReserve(NewMax, NumBytesPerElement, true);
		}

		UE_FORCEINLINE_HINT SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			// If the elements use less space than the inline allocation, only use the inline allocation as slack.
			return (NewMax <= NumInlineElements) ? NumInlineElements : DefaultCalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, true);
		}

		UE_FORCEINLINE_HINT SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			// If the elements use less space than the inline allocation, only use the inline allocation as slack.
			return (NewMax <= NumInlineElements) ? NumInlineElements : DefaultCalculateSlackGrow(NewMax, CurrentMax, NumBytesPerElement, true);
		}

		SIZE_T GetAllocatedSize(SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return HasAllocation() ? (CurrentMax * NumBytesPerElement) : 0;
		}

		UE_FORCEINLINE_HINT bool HasAllocation() const
		{
			return Data != GetInlineElements();
		}

		SizeType GetInitialCapacity() const
		{
			return NumInlineElements;
		}

#if UE_ENABLE_ARRAY_SLACK_TRACKING
		UE_FORCEINLINE_HINT void SlackTrackerLogNum(SizeType NewNumUsed)
		{
			if (HasAllocation())
			{
				FArraySlackTrackingHeader* TrackingHeader = (FArraySlackTrackingHeader*)((uint8*)Data - sizeof(FArraySlackTrackingHeader));

				TrackingHeader->UpdateNumUsed((int64)NewNumUsed);
			}
		}
#endif

	private:
		ForElementType(const ForElementType&) = delete;
		ForElementType& operator=(const ForElementType&) = delete;

		/** The data is allocated through the indirect allocation policy if more than NumInlineElements is needed. */
		ElementType* Data;

		/** The data is stored in this array if less than NumInlineElements is needed. */
		TTypeCompatibleBytes<ElementType> InlineData[NumInlineElements];

		/** @return the base of the aligned inline element data */
		UE_FORCEINLINE_HINT ElementType* GetInlineElements() const
		{
			return (ElementType*)InlineData;
		}
	};

	typedef void ForAnyElementType;
};

template <uint32 NumInlineElements>
struct TAllocatorTraits<TNonRelocatableInlineAllocator<NumInlineElements>> : TAllocatorTraitsBase<TNonRelocatableInlineAllocator<NumInlineElements>>
{
	enum { SupportsSlackTracking = true };
};

/**
 * The fixed allocation policy allocates up to a specified number of elements in the same allocation as the container.
 * It's like the inline allocator, except it doesn't provide secondary storage when the inline storage has been filled.
 */
template <uint32 NumInlineElements>
class TFixedAllocator
{
public:
	using SizeType = int32;

	enum { NeedsElementType = true };
	enum { RequireRangeCheck = true };
	enum { ShrinkByDefault = false };

	template<typename ElementType>
	class ForElementType
	{
	public:

		/** Default constructor. */
		ForElementType()
		{
		}

		/**
		 * Moves the state of another allocator into this one.
		 * Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		 * @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		 */
		UE_FORCEINLINE_HINT void MoveToEmpty(ForElementType& Other)
		{
			checkSlow(this != &Other);

			// Relocate objects from other inline storage
			RelocateConstructItems<ElementType>((void*)InlineData, Other.GetInlineElements(), NumInlineElements);
		}

		// FContainerAllocatorInterface
		UE_FORCEINLINE_HINT ElementType* GetAllocation() const
		{
			return GetInlineElements();
		}

		void ResizeAllocation(SizeType CurrentNum, SizeType NewMax,SIZE_T NumBytesPerElement)
		{
			// Ensure the requested allocation will fit in the inline data area.
			check(NewMax >= 0 && NewMax <= NumInlineElements);
		}

		UE_FORCEINLINE_HINT SizeType CalculateSlackReserve(SizeType NewMax, SIZE_T NumBytesPerElement) const
		{
			// Ensure the requested allocation will fit in the inline data area.
			check(NewMax <= NumInlineElements);
			return NumInlineElements;
		}
		UE_FORCEINLINE_HINT SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			// Ensure the requested allocation will fit in the inline data area.
			check(CurrentMax <= NumInlineElements);
			return NumInlineElements;
		}
		UE_FORCEINLINE_HINT SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			// Ensure the requested allocation will fit in the inline data area.
			check(NewMax <= NumInlineElements);
			return NumInlineElements;
		}

		SIZE_T GetAllocatedSize(SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return 0;
		}

		bool HasAllocation() const
		{
			return false;
		}

		SizeType GetInitialCapacity() const
		{
			return NumInlineElements;
		}

	private:
		ForElementType(const ForElementType&);
		ForElementType& operator=(const ForElementType&);

		/** The data is stored in this array if less than NumInlineElements is needed. */
		TTypeCompatibleBytes<ElementType> InlineData[NumInlineElements];

		/** @return the base of the aligned inline element data */
		ElementType* GetInlineElements() const
		{
			return (ElementType*)InlineData;
		}
	};

	typedef void ForAnyElementType;
};

// We want these to be correctly typed as int32, but we don't want them to have linkage, so we make them macros
#define NumBitsPerDWORD ((int32)32)
#define NumBitsPerDWORDLogTwo ((int32)5)

class FDefaultBitArrayAllocator;

//
// Sparse array allocation definitions
//

/** Encapsulates the allocators used by a sparse array in a single type. */
template<typename InElementAllocator = FDefaultAllocator,typename InBitArrayAllocator = FDefaultBitArrayAllocator>
class TSparseArrayAllocator
{
public:

	typedef InElementAllocator ElementAllocator;
	typedef InBitArrayAllocator BitArrayAllocator;
};

template <uint32 Alignment = DEFAULT_ALIGNMENT, typename InElementAllocator = TAlignedHeapAllocator<Alignment>,typename InBitArrayAllocator = FDefaultBitArrayAllocator>
class TAlignedSparseArrayAllocator
{
public:
	typedef InElementAllocator ElementAllocator;
	typedef InBitArrayAllocator BitArrayAllocator;
};

/** An inline sparse array allocator that allows sizing of the inline allocations for a set number of elements. */
template<
	uint32 NumInlineElements,
	typename SecondaryAllocator = TSparseArrayAllocator<FDefaultAllocator,FDefaultAllocator>
	>
class TInlineSparseArrayAllocator
{
private:

	/** The size to allocate inline for the bit array. */
	enum { InlineBitArrayDWORDs = (NumInlineElements + NumBitsPerDWORD - 1) / NumBitsPerDWORD};

public:

	typedef TInlineAllocator<NumInlineElements,typename SecondaryAllocator::ElementAllocator>		ElementAllocator;
	typedef TInlineAllocator<InlineBitArrayDWORDs,typename SecondaryAllocator::BitArrayAllocator>	BitArrayAllocator;
};

/** An inline sparse array allocator that doesn't have any secondary storage. */
template <uint32 NumInlineElements>
class TFixedSparseArrayAllocator
{
private:

	/** The size to allocate inline for the bit array. */
	enum { InlineBitArrayDWORDs = (NumInlineElements + NumBitsPerDWORD - 1) / NumBitsPerDWORD};

public:

	typedef TFixedAllocator<NumInlineElements>    ElementAllocator;
	typedef TFixedAllocator<InlineBitArrayDWORDs> BitArrayAllocator;
};

//
// Compact set allocation definitions - It operates entirely with uint8 because it includes its lookup table inline with the overall allocation
//

// If enabled store a copy of the 'key' (key % hashcount - 1, so lookup index) for quick removal
// This takes more space but allows compact sets to be used with garbage reference removal
#if !defined UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE
#define UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE 1
#endif

namespace UE::Core::Private::CompactSetAllocatorHelpers
{
	static constexpr uint8 KeyStorageMultiplier = UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE ? 2 : 1;

	template <uint32 NumInlineElements, int32 ElementSize>
	constexpr int32 CalculateRequiredBytes()
	{
		constexpr uint32 TypeSize = 1 + (NumInlineElements > 0xff) + (NumInlineElements > 0xffff) * 2;
		constexpr uint32 HashSize = NumInlineElements < 8 ? 4 : FGenericPlatformMath::RoundUpToPowerOfTwo((NumInlineElements / 2) + 1);
		constexpr uint32 IndexStorageSize = NumInlineElements * KeyStorageMultiplier;
		return Align(NumInlineElements*ElementSize, 4) + 4 + (IndexStorageSize + HashSize) * TypeSize;
	}
}

/** Regular iterator */
template<typename InElementAllocator = FDefaultAllocator>
struct TCompactSetAllocator
{
	/** CompactSets use raw uint8, so we need extra alignment information for inline allocators */
	template <typename ElementType>
	struct AllocatorAlignment
	{
		static constexpr size_t Value = alignof(typename InElementAllocator::template ForElementType<uint8>);
	};

	template<int32 ElementSize>
	using ElementAllocator = InElementAllocator;
};

template<typename InElementAllocator>
struct TAllocatorTraits<TCompactSetAllocator<InElementAllocator>> : TAllocatorTraitsBase<TCompactSetAllocator<InElementAllocator>>
{
	enum
	{
		SupportsFreezeMemoryImage = TAllocatorTraits<InElementAllocator>::SupportsFreezeMemoryImage,
	};
};

/** An inline allocator that allows sizing of the inline allocations for a set number of elements. */
template<uint32 NumInlineElements, typename SecondaryAllocator = TCompactSetAllocator<>>
struct TInlineCompactSetAllocator
{
	template<int32 ElementSize>
	using ElementAllocator = TInlineAllocator<UE::Core::Private::CompactSetAllocatorHelpers::CalculateRequiredBytes<NumInlineElements, ElementSize>(),typename SecondaryAllocator::template ElementAllocator<ElementSize>>;

	/** CompactSets use raw uint8, so we need extra alignment information for inline allocators */
	template <typename ElementType>
	struct AllocatorAlignment
	{
		static constexpr size_t ElementAlignof = alignof(ElementType);
		static constexpr size_t AllocatorAlignof = alignof(typename ElementAllocator<sizeof(ElementType)>::template ForElementType<uint8>);
		static constexpr size_t Value = FGenericPlatformMath::Max(ElementAlignof, AllocatorAlignof);
	};
};

/** An inline allocator that doesn't have any secondary storage. */
template <uint32 NumInlineElements>
struct TFixedCompactSetAllocator
{
	template<int32 ElementSize>
	using ElementAllocator = TFixedAllocator<UE::Core::Private::CompactSetAllocatorHelpers::CalculateRequiredBytes<NumInlineElements, ElementSize>()>;

	/** CompactSets use raw uint8, so we need extra alignment information for inline allocators */
	template <typename ElementType>
	struct AllocatorAlignment
	{
		static constexpr size_t ElementAlignof = alignof(ElementType);
		static constexpr size_t AllocatorAlignof = alignof(typename ElementAllocator<sizeof(ElementType)>::template ForElementType<uint8>);
		static constexpr size_t Value = FGenericPlatformMath::Max(ElementAlignof, AllocatorAlignof);
	};
};



//
// Set allocation definitions.
//

#if !defined(DEFAULT_NUMBER_OF_ELEMENTS_PER_HASH_BUCKET)
#	define DEFAULT_NUMBER_OF_ELEMENTS_PER_HASH_BUCKET	2
#endif
#define DEFAULT_BASE_NUMBER_OF_HASH_BUCKETS				8
#define DEFAULT_MIN_NUMBER_OF_HASHED_ELEMENTS			4

/** Encapsulates the allocators used by a set in a single type. */
template<
	typename InSparseArrayAllocator               = TSparseArrayAllocator<>,
	typename InHashAllocator                      = TInlineAllocator<1,FDefaultAllocator>,
	uint32   AverageNumberOfElementsPerHashBucket = DEFAULT_NUMBER_OF_ELEMENTS_PER_HASH_BUCKET,
	uint32   BaseNumberOfHashBuckets              = DEFAULT_BASE_NUMBER_OF_HASH_BUCKETS,
	uint32   MinNumberOfHashedElements            = DEFAULT_MIN_NUMBER_OF_HASHED_ELEMENTS
	>
class TSparseSetAllocator
{
public:

	/** Computes the number of hash buckets to use for a given number of elements. */
	static UE_FORCEINLINE_HINT uint32 GetNumberOfHashBuckets(uint32 NumHashedElements)
	{
		if(NumHashedElements >= MinNumberOfHashedElements)
		{
			return FPlatformMath::RoundUpToPowerOfTwo(NumHashedElements / AverageNumberOfElementsPerHashBucket + BaseNumberOfHashBuckets);
		}

		return 1;
	}

	using SparseArrayAllocator = InSparseArrayAllocator;
	using HashAllocator = InHashAllocator;
};

template<
	typename InSparseArrayAllocator,
	typename InHashAllocator,
	uint32   AverageNumberOfElementsPerHashBucket,
	uint32   BaseNumberOfHashBuckets,
	uint32   MinNumberOfHashedElements
>
struct TAllocatorTraits<TSparseSetAllocator<InSparseArrayAllocator, InHashAllocator, AverageNumberOfElementsPerHashBucket, BaseNumberOfHashBuckets, MinNumberOfHashedElements>> :
	TAllocatorTraitsBase<TSparseSetAllocator<InSparseArrayAllocator, InHashAllocator, AverageNumberOfElementsPerHashBucket, BaseNumberOfHashBuckets, MinNumberOfHashedElements>>
{
	enum
	{
		SupportsFreezeMemoryImage = TAllocatorTraits<InSparseArrayAllocator>::SupportsFreezeMemoryImage && TAllocatorTraits<InHashAllocator>::SupportsFreezeMemoryImage,
	};
};

/** An inline set allocator that allows sizing of the inline allocations for a set number of elements. */
template<
	uint32   NumInlineElements,
	typename SecondaryAllocator                   = TSparseSetAllocator<TSparseArrayAllocator<FDefaultAllocator,FDefaultAllocator>,FDefaultAllocator>,
	uint32   AverageNumberOfElementsPerHashBucket = DEFAULT_NUMBER_OF_ELEMENTS_PER_HASH_BUCKET,
	uint32   MinNumberOfHashedElements            = DEFAULT_MIN_NUMBER_OF_HASHED_ELEMENTS
	>
class TInlineSparseSetAllocator
{
private:

	enum { NumInlineHashBuckets = (NumInlineElements + AverageNumberOfElementsPerHashBucket - 1) / AverageNumberOfElementsPerHashBucket };

	static_assert(NumInlineHashBuckets > 0 && !(NumInlineHashBuckets & (NumInlineHashBuckets - 1)), "Number of inline buckets must be a power of two");

public:

	/** Computes the number of hash buckets to use for a given number of elements. */
	static UE_FORCEINLINE_HINT uint32 GetNumberOfHashBuckets(uint32 NumHashedElements)
	{
		const uint32 NumDesiredHashBuckets = FPlatformMath::RoundUpToPowerOfTwo(NumHashedElements / AverageNumberOfElementsPerHashBucket);
		if (NumDesiredHashBuckets < NumInlineHashBuckets)
		{
			return NumInlineHashBuckets;
		}

		if (NumHashedElements < MinNumberOfHashedElements)
		{
			return NumInlineHashBuckets;
		}

		return NumDesiredHashBuckets;
	}

	using SparseArrayAllocator = TInlineSparseArrayAllocator<NumInlineElements,typename SecondaryAllocator::SparseArrayAllocator>;
	using HashAllocator = TInlineAllocator<NumInlineHashBuckets,typename SecondaryAllocator::HashAllocator>;
};

/** An inline set allocator that doesn't have any secondary storage. */
template<
	uint32 NumInlineElements,
	uint32 AverageNumberOfElementsPerHashBucket = DEFAULT_NUMBER_OF_ELEMENTS_PER_HASH_BUCKET,
	uint32 MinNumberOfHashedElements            = DEFAULT_MIN_NUMBER_OF_HASHED_ELEMENTS
	>
class TFixedSparseSetAllocator
{
private:

	enum { NumInlineHashBuckets = (NumInlineElements + AverageNumberOfElementsPerHashBucket - 1) / AverageNumberOfElementsPerHashBucket };

	static_assert(NumInlineHashBuckets > 0 && !(NumInlineHashBuckets & (NumInlineHashBuckets - 1)), "Number of inline buckets must be a power of two");

public:

	/** Computes the number of hash buckets to use for a given number of elements. */
	static UE_FORCEINLINE_HINT uint32 GetNumberOfHashBuckets(uint32 NumHashedElements)
	{
		const uint32 NumDesiredHashBuckets = FPlatformMath::RoundUpToPowerOfTwo(NumHashedElements / AverageNumberOfElementsPerHashBucket);
		if (NumDesiredHashBuckets < NumInlineHashBuckets)
		{
			return NumInlineHashBuckets;
		}

		if (NumHashedElements < MinNumberOfHashedElements)
		{
			return NumInlineHashBuckets;
		}

		return NumDesiredHashBuckets;
	}

	using SparseArrayAllocator = TFixedSparseArrayAllocator<NumInlineElements>;
	using HashAllocator = TFixedAllocator<NumInlineHashBuckets>;
};


/**
 * 'typedefs' for various allocator defaults.
 *
 * These should be replaced with actual typedefs when Core.h include order is sorted out, as then we won't need to
 * 'forward' these TAllocatorTraits specializations below.
 */

class FDefaultSparseSetAllocator   : public TSparseSetAllocator<>   { public: typedef TSparseSetAllocator<>   Typedef; };
class FDefaultCompactSetAllocator  : public TCompactSetAllocator<>  { public: typedef TCompactSetAllocator<>  Typedef; };
class FDefaultBitArrayAllocator    : public TInlineAllocator<4>     { public: typedef TInlineAllocator<4>     Typedef; };
class FDefaultSparseArrayAllocator : public TSparseArrayAllocator<> { public: typedef TSparseArrayAllocator<> Typedef; };

#ifndef UE_USE_COMPACT_SET_AS_DEFAULT
#define UE_USE_COMPACT_SET_AS_DEFAULT 0
#endif

#if UE_USE_COMPACT_SET_AS_DEFAULT
class FDefaultSetAllocator         : public TCompactSetAllocator<>  { public: typedef TCompactSetAllocator<>  Typedef; };
template<typename InSparseArrayAllocator = TSparseArrayAllocator<>, typename InHashAllocator = TInlineAllocator<1,FDefaultAllocator>, uint32... N>
class TSetAllocator        		   : public TCompactSetAllocator<InHashAllocator>  { public: typedef TCompactSetAllocator<InHashAllocator>  Typedef; };
template <uint32 N, typename S = TCompactSetAllocator<>, uint32... NN>
class TInlineSetAllocator          : public TInlineCompactSetAllocator<N, S>  { public: typedef TInlineCompactSetAllocator<N, S>  Typedef; };
template <uint32 N, uint32... Y>
class TFixedSetAllocator           : public TFixedCompactSetAllocator<N>  { public: typedef TFixedCompactSetAllocator<N>  Typedef; };
#else
class FDefaultSetAllocator         : public TSparseSetAllocator<>         { public: typedef TSparseSetAllocator<>         Typedef; };
template<typename InSparseArrayAllocator = TSparseArrayAllocator<>, typename InHashAllocator = TInlineAllocator<1,FDefaultAllocator>, uint32... N>
class TSetAllocator         	   : public TSparseSetAllocator<InSparseArrayAllocator, InHashAllocator, N...>  { public: typedef TSparseSetAllocator<InSparseArrayAllocator, InHashAllocator, N...>  Typedef; };
template<uint32 N, typename S = TSparseSetAllocator<TSparseArrayAllocator<FDefaultAllocator,FDefaultAllocator>,FDefaultAllocator>, uint32... NN>
class TInlineSetAllocator          : public TInlineSparseSetAllocator<N, S, NN...>  { public: typedef TInlineSparseSetAllocator<N, S, NN...>  Typedef; };
template <uint32... N>
class TFixedSetAllocator           : public TFixedSparseSetAllocator<N...>  { public: typedef TFixedSparseSetAllocator<N...>  Typedef; };
#endif

template <int IndexSize> struct TAllocatorTraits<TSizedDefaultAllocator<IndexSize>> : TAllocatorTraits<typename TSizedDefaultAllocator<IndexSize>::Typedef> {};

template <> struct TAllocatorTraits<FDefaultAllocator>            : TAllocatorTraits<typename FDefaultAllocator           ::Typedef> {};
template <> struct TAllocatorTraits<FDefaultSetAllocator>         : TAllocatorTraits<typename FDefaultSetAllocator        ::Typedef> {};
template <> struct TAllocatorTraits<FDefaultSparseSetAllocator>   : TAllocatorTraits<typename FDefaultSparseSetAllocator  ::Typedef> {};
template <> struct TAllocatorTraits<FDefaultBitArrayAllocator>    : TAllocatorTraits<typename FDefaultBitArrayAllocator   ::Typedef> {};
template <> struct TAllocatorTraits<FDefaultSparseArrayAllocator> : TAllocatorTraits<typename FDefaultSparseArrayAllocator::Typedef> {};

template <typename InElementAllocator, typename InBitArrayAllocator>
struct TAllocatorTraits<TSparseArrayAllocator<InElementAllocator, InBitArrayAllocator>> : TAllocatorTraitsBase<TSparseArrayAllocator<InElementAllocator, InBitArrayAllocator>>
{
	enum
	{
		SupportsFreezeMemoryImage = TAllocatorTraits<InElementAllocator>::SupportsFreezeMemoryImage && TAllocatorTraits<InBitArrayAllocator>::SupportsFreezeMemoryImage,
	};
};

template <uint8 FromIndexSize, uint8 ToIndexSize> struct TCanMoveBetweenAllocators<TSizedDefaultAllocator<FromIndexSize>, TSizedDefaultAllocator<ToIndexSize>> : TCanMoveBetweenAllocators<typename TSizedDefaultAllocator<FromIndexSize>::Typedef, typename TSizedDefaultAllocator<ToIndexSize>::Typedef> {};
