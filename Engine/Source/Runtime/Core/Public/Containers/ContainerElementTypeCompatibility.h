// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UnrealTypeTraits.h"

/**
 * Temporary compatibility mechanism to be used solely for the purpose of raw pointers to wrapped pointers.
 * Specialization and use of this type is not supported beyond the specific instances in ObjectPtr.h.
 * These types and the container methods that leverage them will be removed in a future release.
 */
template <typename InElementType>
struct TContainerElementTypeCompatibility
{
	typedef InElementType ReinterpretType;
	typedef InElementType CopyFromOtherType;

	template <typename IterBeginType, typename IterEndType, typename OperatorType = InElementType&(*)(IterBeginType&)>
	static void ReinterpretRange(IterBeginType Iter, IterEndType IterEnd, OperatorType Operator = [](IterBeginType& InIt) -> InElementType& { return *InIt; })
	{
	}

	template <typename IterBeginType, typename IterEndType, typename SizeType, typename OperatorType = InElementType & (*)(IterBeginType&)>
	static void ReinterpretRangeContiguous(IterBeginType Iter, IterEndType IterEnd, SizeType Size, OperatorType Operator = [](IterBeginType& InIt) -> InElementType& { return *InIt; })
	{
	}

	static constexpr void CopyingFromOtherType() {}
};

/**
 * Temporary compatibility mechanism to be used solely for the purpose of raw pointers to wrapped pointers.
 * These types and the container methods that leverage them will be removed in a future release.
 */
template <typename ElementType>
constexpr bool TIsContainerElementTypeReinterpretable_V = !std::is_same_v<typename TContainerElementTypeCompatibility<ElementType>::ReinterpretType, ElementType>;

/**
 * Temporary compatibility mechanism to be used solely for the purpose of raw pointers to wrapped pointers.
 * These types and the container methods that leverage them will be removed in a future release.
 */
template <typename ElementType>
constexpr bool TIsContainerElementTypeCopyable_V = !std::is_same_v<typename TContainerElementTypeCompatibility<ElementType>::CopyFromOtherType, ElementType>;
