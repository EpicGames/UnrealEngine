// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::Core::Private
{
	[[noreturn]] CORE_API void OnInvalidArrayNum(unsigned long long NewNum);
	[[noreturn]] CORE_API void OnInvalidSetNum  (unsigned long long NewNum);
}
