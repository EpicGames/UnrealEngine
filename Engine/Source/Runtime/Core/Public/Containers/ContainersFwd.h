// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "CoreTypes.h"
#include "Traits/IsContiguousContainer.h"

// IWYU pragma: begin_exports

/// @cond DOXYGEN_WARNINGS
template<int IndexSize> class TSizedDefaultAllocator;
template<int IndexSize> class TSizedNonshrinkingAllocator;
template<typename ElementType, bool bInAllowDuplicateKeys = false> struct DefaultKeyFuncs;

using FDefaultAllocator = TSizedDefaultAllocator<32>;
using FNonshrinkingAllocator = TSizedNonshrinkingAllocator<32>;
using FDefaultAllocator64 = TSizedDefaultAllocator<64>;
class FDefaultSetAllocator;
class FDefaultCompactSetAllocator;
class FDefaultSparseSetAllocator;

class FString;
class FAnsiString;
class FUtf8String;

// FWideString is an alias for FString.
//
// This is so ANSICHAR/UTF8CHAR/WIDECHAR can be matched with FAnsiString/FUtf8String/FWideString when overloaded or specialized.
//
// FWideString should be the 'real' string class and FString should be the alias, but can't be for legacy reasons.
// Forward declarations of FString expect it to be a class and changing it would affect ABIs and bloat PDBs.
using FWideString = FString;

template<> struct TIsContiguousContainer<FString>     { static constexpr bool Value = true; };
template<> struct TIsContiguousContainer<FAnsiString> { static constexpr bool Value = true; };
template<> struct TIsContiguousContainer<FUtf8String> { static constexpr bool Value = true; };

namespace UE::Core::Private
{
	template <typename CharType>
	struct TCharTypeToStringType;

	template <>
	struct TCharTypeToStringType<WIDECHAR>
	{
		// TCHAR has been wide for too long to fix
		static_assert(sizeof(TCHAR) == sizeof(WIDECHAR), "TCHAR is expected to be wide");
		using Type = FWideString;
	};

	template <>
	struct TCharTypeToStringType<ANSICHAR>
	{
		using Type = FAnsiString;
	};

	template <>
	struct TCharTypeToStringType<UTF8CHAR>
	{
		using Type = FUtf8String;
	};
}

template <typename CharType>
using TString = typename UE::Core::Private::TCharTypeToStringType<CharType>::Type;

template<typename T, typename Allocator = FDefaultAllocator> class TArray;
template<typename T> using TArray64 = TArray<T, FDefaultAllocator64>;
template<typename T, typename SizeType = int32> class TArrayView;
template<typename T> using TArrayView64 = TArrayView<T, int64>;
template<typename T, typename SizeType = int32> using TConstArrayView = TArrayView<const T, SizeType>;
template<typename T> using TConstArrayView64 = TConstArrayView<T, int64>;
template<typename T> class TTransArray;
template<typename InKeyType, typename InValueType, bool bInAllowDuplicateKeys> struct TDefaultMapHashableKeyFuncs;
template<typename InKeyType, typename InValueType, typename SetAllocator = FDefaultSparseSetAllocator, typename KeyFuncs = TDefaultMapHashableKeyFuncs<InKeyType, InValueType, false> > class TSparseMap;
template<typename InKeyType, typename InValueType, typename SetAllocator = FDefaultCompactSetAllocator, typename KeyFuncs = TDefaultMapHashableKeyFuncs<InKeyType, InValueType, false> > class TCompactMap;
template<typename InKeyType, typename InValueType, typename SetAllocator = FDefaultSetAllocator, typename KeyFuncs = TDefaultMapHashableKeyFuncs<InKeyType, InValueType, false> > class TMap;
template<typename KeyType, typename ValueType, typename SetAllocator = FDefaultSparseSetAllocator, typename KeyFuncs = TDefaultMapHashableKeyFuncs<KeyType, ValueType, true > > class TSparseMultiMap;
template<typename KeyType, typename ValueType, typename SetAllocator = FDefaultCompactSetAllocator, typename KeyFuncs = TDefaultMapHashableKeyFuncs<KeyType, ValueType, true > > class TCompactMultiMap;
template<typename KeyType, typename ValueType, typename SetAllocator = FDefaultSetAllocator, typename KeyFuncs = TDefaultMapHashableKeyFuncs<KeyType, ValueType, true > > class TMultiMap;
template <typename T = void > struct TLess;
template <typename> struct TTypeTraits;
template<typename InKeyType, typename InValueType, typename ArrayAllocator = FDefaultAllocator, typename SortPredicate = TLess<typename TTypeTraits<InKeyType>::ConstPointerType> > class TSortedMap;
template <typename InElementType, typename ArrayAllocator = FDefaultAllocator, typename SortPredicate = TLess<typename TTypeTraits<InElementType>::ConstPointerType>> class TSortedSet;
template<typename InElementType, typename KeyFuncs = DefaultKeyFuncs<InElementType>, typename Allocator = FDefaultSparseSetAllocator> class TSparseSet;
template<typename InElementType, typename KeyFuncs = DefaultKeyFuncs<InElementType>, typename Allocator = FDefaultCompactSetAllocator> class TCompactSet;
template<typename InElementType, typename KeyFuncs = DefaultKeyFuncs<InElementType>, typename Allocator = FDefaultSetAllocator> class TSet;
template<typename InElementType, typename InSizeType = int32> class TStridedView;
template<typename T, typename SizeType = int32> using TConstStridedView = TStridedView<const T, SizeType>;
/// @endcond

// IWYU pragma: end_exports
