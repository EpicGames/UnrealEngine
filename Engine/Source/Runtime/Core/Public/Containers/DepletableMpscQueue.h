// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Deprecated
UE_DEPRECATED_HEADER(5.2, "#include Containers/DepletableMpmcQueue.h instead of Containers/DepletableMpscQueue.h (/s/Mpsc/Mpmc).")

#include "Containers/DepletableMpmcQueue.h"
