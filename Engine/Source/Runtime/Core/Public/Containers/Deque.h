// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "IteratorAdapter.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Templates/MemoryOps.h"
#include "Traits/IsTriviallyRelocatable.h"

#include <initializer_list>

template <typename InElementType, typename InAllocatorType = FDefaultAllocator>
class TDeque;

namespace UE
{
namespace Deque
{
namespace Private
{
/**
 * This implementation assumes that the Index value is never going to exceed twice the Range value. This way we can
 * avoid the modulo operator or a power of 2 range value requirement and have an efficient wrap around function.
 */
template <typename InSizeType>
UE_FORCEINLINE_HINT InSizeType WrapAround(InSizeType Index, InSizeType Range)
{
	return (Index < Range) ? Index : Index - Range;
}

/**
 * TDeque iterator class.
 */
template <typename InElementType, typename InSizeType>
class TIteratorBase
{
public:
	using ElementType = InElementType;
	using SizeType = InSizeType;

	TIteratorBase() = default;

protected:
	/**
	 * Internal parameter constructor (used exclusively by the container).
	 */
	TIteratorBase(ElementType* Data, SizeType Range, SizeType Offset)
		: Data(Data), Range(Range), Offset(Offset)
	{
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& Dereference() const
	{
		return *(Data + WrapAround(Offset, Range));
	}

	inline void Increment()
	{
		checkSlow(Offset + 1 < Range * 2);	// See WrapAround
		Offset++;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool Equals(const TIteratorBase& Other) const
	{
		return Data + Offset == Other.Data + Other.Offset;
	}

private:
	ElementType* Data = nullptr;
	SizeType Range = 0;
	SizeType Offset = 0;
};

template <typename InElementType, typename InSizeType>
using TIterator = TIteratorAdapter<TIteratorBase<InElementType, InSizeType>>;
}  // namespace Private
}  // namespace Deque
}  // namespace UE

/**
 * Sequential double-ended queue (deque) container class.
 *
 * A dynamically sized sequential queue of arbitrary size.
 **/
template <typename InElementType, typename InAllocatorType>
class TDeque
{
	template <typename AnyElementType, typename AnyAllocatorType>
	friend class TDeque;

public:
	using AllocatorType = InAllocatorType;
	using SizeType = typename InAllocatorType::SizeType;
	using ElementType = InElementType;

	using ElementAllocatorType = std::conditional_t<
		AllocatorType::NeedsElementType,
		typename AllocatorType::template ForElementType<ElementType>,
		typename AllocatorType::ForAnyElementType>;

	using ConstIteratorType = UE::Deque::Private::TIterator<const ElementType, SizeType>;
	using IteratorType = UE::Deque::Private::TIterator<ElementType, SizeType>;

	[[nodiscard]] TDeque() : Capacity(Storage.GetInitialCapacity())
	{
	}

	[[nodiscard]] TDeque(TDeque&& Other)
	{
		MoveUnchecked(MoveTemp(Other));
	}

	[[nodiscard]] TDeque(const TDeque& Other)
		: Capacity(Storage.GetInitialCapacity())
	{
		CopyUnchecked(Other);
	}

	[[nodiscard]] TDeque(std::initializer_list<ElementType> InList)
		: Capacity(Storage.GetInitialCapacity())
	{
		CopyUnchecked(InList);
	}

	~TDeque()
	{
		UE_STATIC_ASSERT_WARN(TIsTriviallyRelocatable_V<InElementType>, "TArray can only be used with trivially relocatable types");

		Empty();
	}

	TDeque& operator=(TDeque&& Other)
	{
		if (this != &Other)
		{
			Reset();
			MoveUnchecked(MoveTemp(Other));
		}
		return *this;
	}

	TDeque& operator=(const TDeque& Other)
	{
		if (this != &Other)
		{
			Reset();
			CopyUnchecked(Other);
		}
		return *this;
	}

	TDeque& operator=(std::initializer_list<ElementType> InList)
	{
		Reset();
		CopyUnchecked(InList);
		return *this;
	}

	[[nodiscard]] inline const ElementType& operator[](SizeType Index) const
	{
		CheckValidIndex(Index);
		return GetData()[UE::Deque::Private::WrapAround(Head + Index, Capacity)];
	}

	[[nodiscard]] inline ElementType& operator[](SizeType Index)
	{
		CheckValidIndex(Index);
		return GetData()[UE::Deque::Private::WrapAround(Head + Index, Capacity)];
	}

	[[nodiscard]] inline const ElementType& Last() const
	{
		CheckValidIndex(0);
		return GetData()[UE::Deque::Private::WrapAround(Tail + Capacity - 1, Capacity)];
	}

	[[nodiscard]] inline ElementType& Last()
	{
		CheckValidIndex(0);
		return GetData()[UE::Deque::Private::WrapAround(Tail + Capacity - 1, Capacity)];
	}

	[[nodiscard]] inline const ElementType& First() const
	{
		CheckValidIndex(0);
		return GetData()[Head];
	}

	[[nodiscard]] inline ElementType& First()
	{
		CheckValidIndex(0);
		return GetData()[Head];
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return !Count;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT SizeType Max() const
	{
		return Capacity;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT SizeType Num() const
	{
		return Count;
	}

	/**
	 * Helper function to return the amount of memory allocated by this container.
	 * Only returns the size of allocations made directly by the container, not the elements themselves.
	 *
	 * @returns Number of bytes allocated by this container.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT SIZE_T GetAllocatedSize() const
	{
		return Storage.GetAllocatedSize(Capacity, sizeof(ElementType));
	}

	/*
	 * Constructs an element in place using the parameter arguments and adds it at the back of the queue.
	 * This method returns a reference to the constructed element.
	 */
	template <typename... ArgsType>
	ElementType& EmplaceLast(ArgsType&&... Args)
	{
		GrowIfRequired();
		ElementType* Target = GetData() + Tail;
		::new ((void*)Target) ElementType(Forward<ArgsType>(Args)...);
		Tail = UE::Deque::Private::WrapAround(Tail + 1, Capacity);
		Count++;
		return *Target;
	}

	/*
	 * Constructs an element in place using the parameter arguments and adds it at the front of the queue.
	 * This method returns a reference to the constructed element.
	 */
	template <typename... ArgsType>
	ElementType& EmplaceFirst(ArgsType&&... Args)
	{
		GrowIfRequired();
		Head = UE::Deque::Private::WrapAround(Head + Capacity - 1, Capacity);
		ElementType* Target = GetData() + Head;
		::new ((void*)Target) ElementType(Forward<ArgsType>(Args)...);
		Count++;
		return *Target;
	}

	/*
	 * Adds the parameter element at the back of the queue.
	 */
	UE_FORCEINLINE_HINT void PushLast(const ElementType& Element)
	{
		EmplaceLast(Element);
	}

	UE_FORCEINLINE_HINT void PushLast(ElementType&& Element)
	{
		EmplaceLast(MoveTempIfPossible(Element));
	}

	/*
	 * Adds the parameter element at the front of the queue.
	 */
	UE_FORCEINLINE_HINT void PushFirst(const ElementType& Element)
	{
		EmplaceFirst(Element);
	}

	UE_FORCEINLINE_HINT void PushFirst(ElementType&& Element)
	{
		EmplaceFirst(MoveTempIfPossible(Element));
	}

	/*
	 * Removes the element at the back of the queue.
	 * This method requires a non-empty queue.
	 */
	void PopLast()
	{
		CheckValidIndex(0);
		const SizeType NextTail = UE::Deque::Private::WrapAround(Tail + Capacity - 1, Capacity);
		DestructItem(GetData() + NextTail);
		Tail = NextTail;
		Count--;
	}

	/*
	 * Removes the element at the front of the queue.
	 * This method requires a non-empty queue.
	 */
	void PopFirst()
	{
		CheckValidIndex(0);
		DestructItem(GetData() + Head);
		Head = UE::Deque::Private::WrapAround(Head + 1, Capacity);
		Count--;
	}

	/*
	 * Removes the element at the back of the queue if existent after copying it to the parameter
	 * out value.
	 */
	[[nodiscard]] bool TryPopLast(ElementType& OutValue)
	{
		if (IsEmpty())
		{
			return false;
		}
		const SizeType NextTail = UE::Deque::Private::WrapAround(Tail + Capacity - 1, Capacity);
		OutValue = MoveTempIfPossible(GetData()[NextTail]);
		DestructItem(GetData() + NextTail);
		Tail = NextTail;
		Count--;
		return true;
	}

	/*
	 * Removes the element at the front of the queue if existent after copying it to the parameter
	 * out value.
	 */
	[[nodiscard]] bool TryPopFirst(ElementType& OutValue)
	{
		if (IsEmpty())
		{
			return false;
		}
		OutValue = MoveTempIfPossible(GetData()[Head]);
		DestructItem(GetData() + Head);
		Head = UE::Deque::Private::WrapAround(Head + 1, Capacity);
		Count--;
		return true;
	}

	/*
	 * Destroys all contained elements but doesn't release the container's storage.
	 */
	void Reset()
	{
		if (Count)
		{
			if (Head < Tail)
			{
				DestructItems(GetData() + Head, Count);
			}
			else
			{
				DestructItems(GetData(), Tail);
				DestructItems(GetData() + Head, Capacity - Head);
			}
		}
		Head = Tail = Count = 0;
	}

	/*
	 * Empties the queue effectively destroying all contained elements and releases any acquired storage.
	 */
	void Empty()
	{
		Reset();
		if (Capacity)
		{
			Storage.ResizeAllocation(0, 0, sizeof(ElementType));
			Capacity = Storage.GetInitialCapacity();
		}
	}

	/*
	 * Reserves storage for the parameter element count.
	 */
	void Reserve(SizeType InCount)
	{
		if (Capacity < InCount)
		{
			Grow(Storage.CalculateSlackReserve(InCount, sizeof(ElementType)));
		}
	}

	/*
	 * STL IteratorType model compliance methods.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT ConstIteratorType begin() const
	{
		return GetIterator(Head);
	}
	[[nodiscard]] UE_FORCEINLINE_HINT IteratorType begin()
	{
		return GetIterator(Head);
	}
	[[nodiscard]] UE_FORCEINLINE_HINT ConstIteratorType end() const
	{
		return GetIterator(Head + Count);
	}
	[[nodiscard]] UE_FORCEINLINE_HINT IteratorType end()
	{
		return GetIterator(Head + Count);
	}

private:
	ElementAllocatorType Storage;
	SizeType Capacity = 0;
	SizeType Count = 0;
	SizeType Head = 0;
	SizeType Tail = 0;

	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType* GetData() const
	{
		return (const ElementType*)Storage.GetAllocation();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType* GetData()
	{
		return (ElementType*)Storage.GetAllocation();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ConstIteratorType GetIterator(SizeType HeadOffset) const
	{
		return ConstIteratorType(InPlace, GetData(), Max(), HeadOffset);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT IteratorType GetIterator(SizeType HeadOffset)
	{
		return IteratorType(InPlace, GetData(), Max(), HeadOffset);
	}

	/**
	 * Grows the container's storage to the parameter capacity value.
	 * This method preserves existing data.
	 */
	void Grow(SizeType InCapacity)
	{
		checkSlow(Capacity < InCapacity);
		if (Count)
		{
			Linearize();
		}
		Storage.ResizeAllocation(Count, InCapacity, sizeof(ElementType));
		Capacity = InCapacity;
		Head = 0;
		Tail = Count;
	}

	/**
	 * Grows the container to the next capacity value (determined by the storage allocator) if full.
	 * This method preserves existing data.
	 */
	void GrowIfRequired()
	{
		if (Count == Capacity)
		{
			Grow(Storage.CalculateSlackGrow(Count + 1, Capacity, sizeof(ElementType)));
		}
	}

	/*
	 * Copies the parameter container into this one.
	 * This method assumes no previously existing content.
	 */
	void CopyUnchecked(const TDeque& Other)
	{
		checkSlow(!Count);
		if (Other.Count)
		{
			Reserve(Other.Count);
			CopyElements(Other);
		}
		else
		{
			// Retain any inline capacity.
			Capacity = Other.Storage.GetInitialCapacity();
		}
	}

	void CopyUnchecked(std::initializer_list<ElementType> InList)
	{
		const SizeType InCount = static_cast<SizeType>(InList.size());
		checkSlow(!Count);
		if (InCount)
		{
			Reserve(InCount);
			ConstructItems<ElementType>(GetData(), &*InList.begin(), InCount);
			Tail = Count = InCount;
		}
		else
		{
			Capacity = Storage.GetInitialCapacity();
		}
	}

	/*
	 * Moves the parameter container into this one.
	 * This method assumes no previously existing content.
	 */
	void MoveUnchecked(TDeque&& Other)
	{
		checkSlow(!Count);
		if (Other.Count)
		{
			Storage.MoveToEmpty(Other.Storage);
			Capacity = Other.Capacity;
			Count = Other.Count;
			Head = Other.Head;
			Tail = Other.Tail;
			Other.Capacity = Other.Storage.GetInitialCapacity();
			Other.Count = Other.Head = Other.Tail = 0;
		}
		else
		{
			Capacity = Other.Storage.GetInitialCapacity();
		}
	}

	/*
	 * Copies the parameter container elements into this one.
	 * The copied range is linearized.
	 */
	void CopyElements(const TDeque& Other)
	{
		if (Other.Head < Other.Tail)
		{
			ConstructItems<ElementType>(GetData(), Other.GetData() + Other.Head, Other.Count);
		}
		else
		{
			const SizeType HeadToEndOffset = Other.Capacity - Other.Head;
			ConstructItems<ElementType>(GetData(), Other.GetData() + Other.Head, HeadToEndOffset);
			ConstructItems<ElementType>(GetData() + HeadToEndOffset, Other.GetData(), Other.Tail);
		}
		Count = Other.Count;
		Head = 0;
		Tail = UE::Deque::Private::WrapAround(Count, Capacity); 
	}

	/**
	 * Shifts the contained range to the beginning of the storage so it's linear.
	 * This method is faster than a full range rotation but requires a temporary extra storage whenever the tail is
	 * wrapped around.
	 */
	void Linearize()
	{
		if (Head < Tail)
		{
			ShiftElementsLeft(Count);
		}
		else
		{
			ElementAllocatorType TempStorage;
			TempStorage.ResizeAllocation(0, Tail, sizeof(ElementType));
			RelocateConstructItems<ElementType>(TempStorage.GetAllocation(), GetData(), Tail);
			const SizeType HeadToEndOffset = Capacity - Head;
			ShiftElementsLeft(HeadToEndOffset);
			RelocateConstructItems<ElementType>(GetData() + HeadToEndOffset, (ElementType*)TempStorage.GetAllocation(), Tail);
		}
	}

	/**
	 * Moves the parameter number of elements to the left shifting the head to the beginning of the storage.
	 */
	void ShiftElementsLeft(SizeType InCount)
	{
		if (Head == 0)
		{
			return;
		}
		SizeType Offset = 0;
		while (Offset < InCount)
		{
			const SizeType Step = FMath::Min(Head, InCount - Offset);
			RelocateConstructItems<ElementType>(GetData() + Offset, GetData() + Head + Offset, Step);
			Offset += Step;
		}
	}

	inline void CheckValidIndex(SizeType Index) const
	{
		checkSlow((Count >= 0) & (Capacity >= Count));
		if constexpr (AllocatorType::RequireRangeCheck)
		{
			checkf((Index >= 0) & (Index < Count), TEXT("Parameter index %d exceeds container size %d"), Index, Count);
		}
	}

	//---------------------------------------------------------------------------------------------------------------------
	// TDeque comparison operators
	//---------------------------------------------------------------------------------------------------------------------
public:
	[[nodiscard]] inline bool operator==(const TDeque& Right) const
	{
		if (Num() == Right.Num())
		{
			auto EndIt = end();
			auto LeftIt = begin();
			auto RightIt = Right.begin();
			while (LeftIt != EndIt)
			{
				if (*LeftIt++ != *RightIt++)
				{
					return false;
				}
			}
			return true;
		}
		return false;
	}

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	[[nodiscard]] inline bool operator!=(const TDeque& Right) const
	{
		return !(*this == Right);
	}
#endif
};
