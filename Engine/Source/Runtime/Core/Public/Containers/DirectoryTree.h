// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrowWrapper.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "Templates/Tuple.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/UniquePtr.h"

namespace UE::DirectoryTree
{

CORE_API void FixupPathSeparator(FStringBuilderBase& InOutPath, int32 StartIndex, TCHAR InSeparatorChar);
CORE_API int32 FindInsertionIndex(int32 NumChildNodes, const TUniquePtr<FString[]>& RelPaths,
	FStringView FirstPathComponent, bool& bOutExists);

} // namespace UE::DirectoryTree

enum class EDirectoryTreeGetFlags
{
	None = 0,
	/**
	 * If Recursive flag is present, GetChildren will return direct subpaths of a discovered directory and their
	 * transitive subpaths. If false, it will return only the direct subpaths.
	 *
	 * Recursive=false and ImpliedChildren=false is an exception to this simple definition. In that case the
	 * reported results for a requested directory will include the highest level childpaths under it that have been
	 * added to the tree. These may be in transitive subpaths of the parent directory, and in that case their
	 * parent directories in between the requested directory and their path will not be reported because they are
	 * implied directories.
	 */
	Recursive = 0x1,
	/**
	 * If ImpliedParent flag is present, then the requested directory will return results even if it is an
	 * implied directory (directory with child paths but not added itself, @see TDirectoryTree).
	 * If not present, only directories that have been added to the tree will return non-empty results.
	 */
	ImpliedParent = 0x1 << 1,
	/**
	 * If ImpliedChildren is present, then all child paths discovered (either direct or recursive, depending on whether
	 * Recursive flag is present) will be reported in the results, even if they are implied directories (directory
	 * with child paths but not added itself, @see TDirectoryTree). If not present, only files and directories that
	 * have been added to the tree will be returned in the results.
	 */
	ImpliedChildren = 0x1 << 2,
};
ENUM_CLASS_FLAGS(EDirectoryTreeGetFlags);

/**
 * Container for path -> value that can efficiently report whether a parent directory of a given path exists.
 * Supports relative and absolute paths.
 * Supports LongPackageNames and LocalPaths.
 * 
 * Note about Value comparisons:
 * Case-insensitive
 * / is treated as equal to \
 * Presence or absence of terminating separator (/) is ignored in the comparison.
 * Directory elements of . and .. are currently not interpreted and are treated as literal characters.
 *    Callers should not rely on this behavior as it may be corrected in the future.
 *    callers should instead conform the paths before calling.
 * Relative paths and absolute paths are not resolved, and relative paths will never equal absolute paths.
 *    Callers should not rely on this behavior as it may be corrected in the future;
 *    callers should instead conform the paths before calling.
 * 
 * For functions that find parent paths, parent paths are only discovered if they are conformed to the same format as
 * the given path: both paths must be either relative or absolute.
 * 
 * For functions that return Values by reference or by pointer, that reference or pointer can be invalidated
 * by any functions that modify the tree, and should be discarded before calling any such functions.
 * 
 * Some functions that report results for directories behave differently for added directories versus implied
 * directories. An added directory is one that was added specifically via FindOrAdd or other mutators. An implied
 * directory is a directory that is not added, but that has a child path that is added to the tree.
 */
template <typename ValueType>
class TDirectoryTree
{
public:
	template <typename ViewedValueType> struct TIterator;
	using FIterator = TIterator<ValueType>;
	using FConstIterator = TIterator<const ValueType>;
	template <typename ViewedValueType> struct TPointerIterator;
	using FPointerIterator = TPointerIterator<ValueType>;
	using FConstPointerIterator = TPointerIterator<const ValueType>;
	struct FIterationSentinel;
private:
	struct FIteratorInternal;
	struct FTreeNode;

public:
	TDirectoryTree();

	TDirectoryTree(const TDirectoryTree& Other) = default;
	TDirectoryTree& operator=(const TDirectoryTree& Other) = default;

	TDirectoryTree(TDirectoryTree&& Other) = default;
	TDirectoryTree& operator=(TDirectoryTree&& Other) = default;

	/**
	 * Add a path to the tree if it does not already exist. Construct default Value for it if it did not already exist.
	 * Return a reference to the added or existing Value. Optionally report whether the path already existed.
	 * This reference can be invalidated by any operations that modify the tree.
	 */
	ValueType& FindOrAdd(FStringView Path, bool* bOutExisted = nullptr);
	/** Remove all paths and all memory usage from the tree. */
	void Empty();
	/** Remove a path from the tree and optionally report whether it existed. */
	void Remove(FStringView Path, bool* bOutExisted = nullptr);
	/** Free unused slack memory throughout the tree by reallocating containers tightly to their current size. */
	void Shrink();

	/** Return true if no paths are in the tree. */
	bool IsEmpty() const;
	/** Return the number of paths in the tree. */
	int32 Num() const;
	/** How much memory is used by *this, not counting sizeof(*this). */
	SIZE_T GetAllocatedSize() const;

	/** Return whether the given path has been added to the tree. */
	bool Contains(FStringView Path) const;
	/** Return a const pointer to the Value set for the given path, or null if it does not exist. */
	const ValueType* Find(FStringView Path) const;
	/** Return a pointer to the Value set for the given path, or null if it does not exist. */
	ValueType* Find(FStringView Path);
	/** Return whether the given path or any of its parent paths exist in the tree. */
	bool ContainsPathOrParent(FStringView Path) const;
	/**
	 * Return a const pointer to the path's value if it exists, or to its closest parent path's value,
	 * if any of them exist. Otherwise return nullptr.
	 */
	const ValueType* FindClosestValue(FStringView Path) const;
	/**
	 * Return a pointer to the path's value if it exists, or to its closest parent path's value,
	 * if any of them exist. Otherwise return nullptr.
	 */
	ValueType* FindClosestValue(FStringView Path);
	/**
	 * Return whether the given path or any of its parent paths exist in the tree. If it exists,
	 * return the discovered path, and optionally return a const pointer to the value of that path.
	 */
	bool TryFindClosestPath(FStringView Path, FStringBuilderBase& OutPath, const ValueType** OutValue = nullptr) const;
	/**
	 * Return whether the given path or any of its parent paths exist in the tree. If it exists,
	 * return the discovered path, and optionally return a pointer to the value of that path.
	 */
	bool TryFindClosestPath(FStringView Path, FStringBuilderBase& OutPath, ValueType** OutValue);
	/**
	 * Return whether the given path or any of its parent paths exist in the tree. If it exists,
	 * return the discovered path, and optionally return a const pointer to the value of that path.
	 */
	bool TryFindClosestPath(FStringView Path, FString& OutPath, const ValueType** OutValue = nullptr) const;
	/**
	 * Return whether the given path or any of its parent paths exist in the tree. If it exists,
	 * return the discovered path, and optionally return a pointer to the value of that path.
	 */
	bool TryFindClosestPath(FStringView Path, FString& OutPath, ValueType** OutValue);

	/**
	 * Return whether any children of the given path have been added to the tree.
	 */
	bool ContainsChildPaths(FStringView Path) const;

	/**
	 * Report the children (optionally recursive or not, optionally implied or not) in the tree of a given Path
	 * (optionally skipped if implied). @see EDirectoryTreeGetFlags.
	 * Relative paths of discovered children will be appended to OutRelativeChildNames.
	 * 
	 * @return true iff (the path is found in the tree and either it is an added path or ImpliedParent was requested).
	 */
	bool TryGetChildren(FStringView Path, TArray<FString>& OutRelativeChildNames,
		EDirectoryTreeGetFlags Flags = EDirectoryTreeGetFlags::None) const;

	/** Ranged-for accessor for elements in the tree. Same behavior as @see CreateIterator. */
	FIterator begin();
	FConstIterator begin() const;
	FIterationSentinel end() const;

	/** Iterator accessor for path,value pairs that were added to the tree. @see FIterator. */
	FIterator CreateIterator();
	FConstIterator CreateConstIterator() const;

	/**
	 * Iterator accessor for all paths, added or implied, in the tree. For paths that were added to the tree, the
	 * ValueType* on the iterator is non-null and points to the added Value.
	 * For implied paths that are parent directories of child paths in the tree the Value pointer of the iterator is
	 * the value of their closest parent in the tree, or nullptr if none of their parents were added to the tree.
	 * @see FPointerIterator.
	 */
	FPointerIterator CreateIteratorForImplied();
	FConstPointerIterator CreateConstIteratorForImplied() const;

public:
	/**
	 * Sentinel type used to mark the end of ranged-for iteration. Iterator operator!= functions
	 * that take this type return (bool)*this.
	 */
	struct FIterationSentinel
	{
	};

	/**
	 * Iterator used when iterating paths added to the tree, and skipping implied parent directories
	 * that were not added to the tree and have no Value data. The return value of operator* has a (possibly const)
	 * ValueType&.
	 */
	template <typename ViewedValueType>
	struct TIterator
	{
	public:
		TPair<FStringView, ViewedValueType&> operator*() const;
		bool operator!=(const FIterationSentinel&) const;
		TIterator& operator++();

		explicit operator bool() const;

		// Support Iter->Key and Iter->Value.
		TArrowWrapper<TPair<FStringView, ViewedValueType&>> operator->() const;
	private:
		friend class TDirectoryTree<ValueType>;
		TIterator(FTreeNode& Root, EDirectoryTreeGetFlags InFlags, TCHAR InPathSeparator);
		FIteratorInternal Internal;
	};

	/**
	 * Iterator used when iterating all paths in the tree, both added or implied. The return value of
	 * operator* has a (possibly const) ValueType*, which will be non-null for all paths that were added to the tree
	 * and null for the implied parent directories that were not added.
	 */
	template <typename ViewedValueType>
	struct TPointerIterator
	{
	public:
		TPair<FStringView, ViewedValueType*> operator*() const;
		bool operator!=(const FIterationSentinel&) const;
		TPointerIterator& operator++();

		explicit operator bool() const;

		// Support Iter->Key and Iter->Value.
		TArrowWrapper<TPair<FStringView, ViewedValueType*>> operator->() const;
	private:
		friend class TDirectoryTree<ValueType>;
		TPointerIterator(FTreeNode& Root, EDirectoryTreeGetFlags InFlags, TCHAR InPathSeparator);
		FIteratorInternal Internal;
	};

private:

	/**
	 * A tree structure; each node has a sorted array of child paths and a matching array of child nodes.
	 * Child paths are relative paths and are organized by the FirstComponent of their relative path. If there is only
	 * a single child path with a given FirstComponent, the entire relative path to	that child is listed as the
	 * relative path. If there are two or more paths with the same first component, a new child node is created for
	 * the first component, and the paths are then children of that component.
	 * Example:
	 * Root
	 *		/				(FullPath: /)
	 *			A			(FullPath: /A
	 *				X		(FullPath: /A/X
	 *				Y/M		(FullPath: /A/Y/M)
	 *			B/Z			(FullPath: /B/Z)
	 *				N		(FullPath: /B/Z/N)
	 *				O		(FullPath: /B/Z/O)
	 *			C/W/P		(FullPath: /C/W/P)
	 */
	struct FTreeNode
	{
	public:
		FTreeNode() = default;
		FTreeNode(FTreeNode&& Other);
		FTreeNode& operator=(FTreeNode&& Other);
		FTreeNode(const FTreeNode& Other);
		FTreeNode& operator=(const FTreeNode& Other);

		/** Remove Value and ChildNodes, return state to default-constructed state. */
		void Reset();

		int32 GetNumChildNodes() const;

		/**
		 * Recursively search the node's subtree to find the given relative directory name, adding nodes for the path
		 * and its parents if required. Return reference to the added or existing node's value.
		 */
		ValueType& FindOrAdd(FStringView InRelPath, bool& bOutExisted);
		/** Remove the Value if it exists in the tree. */
		void Remove(FStringView InRelPath, bool& bOutExisted);
		/** Return pointer to the Value stored in RelPath, if RelPath exists in the tree. */
		ValueType* Find(FStringView InRelPath);

		/**
		 * Recursively search this node's subtree for the given relative path, then return whether that path has any
		 * children. Returns false if the given relative path doesn't exist in the tree or if it is an explicit path
		 * with no children. Returns true if the given path is implicit (there is a node representing a sub-path of it)
		 * or if it is an explicit path with direct or indirect children.
		 */
		bool ContainsChildPaths(FStringView InRelPath) const;

		bool TryGetChildren(FStringBuilderBase& ReportedPathPrefix, TCHAR InPathSeparator, FStringView InRelPath,
			TArray<FString>& OutRelativeChildNames, EDirectoryTreeGetFlags Flags) const;

		/**
		 * Recursively search the node's subtree to find the given RelPath. Return whether the path or any parent is
		 * found, and if any is found and Outpath is present, append relative path of the discovered path into OutPath.
		 */
		ValueType* TryFindClosestPath(FStringView RelPath, FStringBuilderBase* OutPath, TCHAR SeparatorChar);

		/** Are no paths contained within the node. */
		bool IsEmpty() const;
		/** How much memory is used by *this, not counting sizeof(*this). */
		SIZE_T GetAllocatedSize() const;

		/** Reduce memory used in buffers. */
		void Shrink();

		/**
		 * Report whether the node has a value, which is equivalent to the node's path existing in the DirectoryTree.
		 * Nodes might exist in the tree without their path existing, if they are parent paths that have not been
		 * added.
		 */
		bool HasValue() const;
		/** Get a reference to the node's Value. Invalid to call if !HasValue. */
		ValueType& GetValue();
		/** Get a reference to the node's Value. Invalid to call if !HasValue. */
		const ValueType& GetValue() const;
		/** Set HasValue=true, and move InValue into the node's value, after destructing any existing old value. */
		void SetValue(ValueType&& InValue);
		/** Set HasValue=true, and default-construct the node's value, after destructing any existing old value. */
		void SetDefaultValue();
		/** Set HasValue=false, and destruct any existing old value. */
		void RemoveValue();

		void FixupDirectChildrenPathSeparator(TCHAR OldSeparator, TCHAR NewSeparator);

	private:
		static constexpr uint32 NumFlagBits = 1u;
		static constexpr uint32 FlagsShift = (8u * sizeof(uint32) - NumFlagBits);
		static constexpr uint32 FlagsMask = 0x1u << FlagsShift;
		void SetNumChildNodes(int32 InNumChildNodes);

		/**
		 * Search the sorted ChildNode RelPaths for the given FirstPathComponent, which must be only a single
		 * path component, and return the index either of the existing path or where the path should be inserted
		 * if new. Also report whether it exists.
		 */
		int32 FindInsertionIndex(FStringView FirstPathComponent, bool& bOutExists) const;
		/** Insert the given RelPath and ChildNode at the given index; must be the index from FindInsertionIndex. */
		FTreeNode& InsertChildNode(int32 InsertionIndex, FString&& RelPath, FTreeNode&& ChildNode);
		/** Remove the RelPath and ChildNode from the given index. */
		void RemoveChildNodeAt(int32 RemoveIndex);

		/** Merge the node with its direct child if possible, and if so adjust the input RelPath to match. */
		static void ConditionalCompactNode(FString& RelPath, FTreeNode& ChildNode);
		/** Reallocate the NumChildNodes arrays to match the given new capacity. */
		void Realloc(int32 NewCapacity);

	private:
		friend struct FIteratorInternal;

		TTypeCompatibleBytes<ValueType> Value;
		TUniquePtr<FString[]> RelPaths;
		TUniquePtr<FTreeNode[]> ChildNodes;
		uint32 NumChildNodesAndFlags = 0;
		int32 CapacityChildNodes = 0;
	};

	struct FIteratorInternal
	{
	public:
		FIteratorInternal(FTreeNode& Root, EDirectoryTreeGetFlags InFlags, TCHAR InPathSeparator);
		// Iterators for TDirectoryTree are heavy-weight; they keep a TArray and a TStringBuilder.
		// Do not copy or move them use them only in for loops, or on the heap:
		// for (TDirectoryTree<...>::FIterator Iter(Tree.CreateIterator()); Iter; ++Iter)
		// TUniquePtr<TDirectoryTree<...>::FIterator> IterPtr = MakeUnique<TDirectoryTree<...>::FIterator>(Tree.CreateIterator())
		FIteratorInternal(FIteratorInternal&& Other) = delete;
		FIteratorInternal& operator=(FIteratorInternal&& Other) = delete;
		FIteratorInternal(const FIteratorInternal& Other) = delete;
		FIteratorInternal& operator=(const FIteratorInternal& Other) = delete;

		TPair<FStringView, ValueType*> operator*() const;
		bool operator!=(const FIterationSentinel&) const;
		FIteratorInternal& operator++();

		explicit operator bool() const;

	private:
		struct FStackData
		{
			explicit FStackData(FTreeNode& InNode, ValueType* ParentClosestValue);

			FStringView RemainingChildRelPath;
			FTreeNode* Node;
			ValueType* ClosestValue;
			int32 ChildIndex;
			int32 PathLenBeforeChildNode;
			bool bChildRelPathInitialized;
			bool bChildNodeInitialized;
		};

	private:
		void TraverseToValid();
		void IncrementRemainingChildRelPath(FStackData* Data);

	private:
		TArray<FStackData, TInlineAllocator<10>> Stack;
		TStringBuilder<256> Path;
		EDirectoryTreeGetFlags Flags;
		TCHAR PathSeparator;
	};

private:
	bool NormalizePathForReading(FStringView& Path, FStringBuilderBase& NormalizeBuffer) const;
	bool NormalizePathForWriting(FStringView& Path, FStringBuilderBase& NormalizeBuffer);
	ValueType* TryFindClosestPathInternal(FStringView Path, FStringBuilderBase* OutPath);
	void InitializePathSeparator(TCHAR InPathSeparator);

private:
	FTreeNode Root;
	int32 NumPaths = 0;
	TCHAR PathSeparator = '/';
	bool bPathSeparatorInitialized = false;
	bool bNeedDriveWithoutPathFixup = false;
};


template <typename ValueType>
inline TDirectoryTree<ValueType>::TDirectoryTree()
{
	check(Root.IsEmpty());
}

template <typename ValueType>
ValueType& TDirectoryTree<ValueType>::FindOrAdd(FStringView Path, bool* bOutExisted)
{
	if (Path.IsEmpty())
	{
		if (bOutExisted)
		{
			*bOutExisted = Root.HasValue();
		}
		if (!Root.HasValue())
		{
			Root.SetDefaultValue();
			++NumPaths;
		}
		return Root.GetValue();
	}

	if (!bPathSeparatorInitialized)
	{
		int32 UnusedIndex;
		if (Path.FindChar('/', UnusedIndex))
		{
			InitializePathSeparator('/');
		}
		else if (Path.FindChar('\\', UnusedIndex))
		{
			InitializePathSeparator('\\');
		}
	}

	TStringBuilder<16> NormalizeBuffer;
	NormalizePathForWriting(Path, NormalizeBuffer);

	bool bExisted;
	ValueType& Result = Root.FindOrAdd(Path, bExisted);
	if (!bExisted)
	{
		++NumPaths;
	}

	if (bOutExisted)
	{
		*bOutExisted = bExisted;
	}

	return Result;
}

template <typename ValueType>
inline void TDirectoryTree<ValueType>::Empty()
{
	Root.Reset();
	NumPaths = 0;
	PathSeparator = '/';
	bPathSeparatorInitialized = false;
}

template <typename ValueType>
void TDirectoryTree<ValueType>::Remove(FStringView Path, bool* bOutExisted)
{
	bool bExisted;
	if (Path.IsEmpty())
	{
		bExisted = Root.HasValue();
		if (bExisted)
		{
			Root.RemoveValue();
		}
	}
	else
	{
		TStringBuilder<16> NormalizeBuffer;
		NormalizePathForReading(Path, NormalizeBuffer);

		Root.Remove(Path, bExisted);
	}
	if (bExisted)
	{
		check(NumPaths > 0);
		--NumPaths;
	}
	if (bOutExisted)
	{
		*bOutExisted = bExisted;
	}
}

template <typename ValueType>
inline void TDirectoryTree<ValueType>::Shrink()
{
	Root.Shrink();
}

template <typename ValueType>
inline bool TDirectoryTree<ValueType>::IsEmpty() const
{
	return Root.IsEmpty();
}

template <typename ValueType>
inline int32 TDirectoryTree<ValueType>::Num() const
{
	return NumPaths;
}

template <typename ValueType>
inline SIZE_T TDirectoryTree<ValueType>::GetAllocatedSize() const
{
	return Root.GetAllocatedSize();
}

template <typename ValueType>
inline bool TDirectoryTree<ValueType>::Contains(FStringView Path) const
{
	return Find(Path) != nullptr;
}

template <typename ValueType>
inline const ValueType* TDirectoryTree<ValueType>::Find(FStringView Path) const
{
	return const_cast<TDirectoryTree*>(this)->Find(Path);
}

template <typename ValueType>
inline ValueType* TDirectoryTree<ValueType>::Find(FStringView Path)
{
	if (Path.IsEmpty())
	{
		return Root.HasValue() ? &Root.GetValue() : nullptr;
	}

	TStringBuilder<16> NormalizeBuffer;
	NormalizePathForReading(Path, NormalizeBuffer);

	return Root.Find(Path);
}

template <typename ValueType>
inline bool TDirectoryTree<ValueType>::ContainsPathOrParent(FStringView Path) const
{
	return FindClosestValue(Path) != nullptr;
}

template <typename ValueType>
inline const ValueType* TDirectoryTree<ValueType>::FindClosestValue(FStringView Path) const
{
	return const_cast<TDirectoryTree*>(this)->TryFindClosestPathInternal(Path, nullptr);
}

template <typename ValueType>
inline ValueType* TDirectoryTree<ValueType>::FindClosestValue(FStringView Path)
{
	return TryFindClosestPathInternal(Path, nullptr);
}

template <typename ValueType>
inline bool TDirectoryTree<ValueType>::TryFindClosestPath(FStringView Path, FString& OutPath, const ValueType** OutValue) const
{
	ValueType* ResultValue;
	bool bResult = const_cast<TDirectoryTree*>(this)->TryFindClosestPath(Path, OutPath, &ResultValue);
	if (bResult)
	{
		if (OutValue)
		{
			*OutValue = ResultValue;
		}
	}
	return bResult;
}

template <typename ValueType>
inline bool TDirectoryTree<ValueType>::TryFindClosestPath(FStringView Path, FString& OutPath, ValueType** OutValue)
{
	TStringBuilder<1024> Builder;
	OutPath.Reset(); // Reset the path even if we fail, to match the behavior when taking a StringBuilderBase
	ValueType* Result = TryFindClosestPathInternal(Path, &Builder);
	if (Result)
	{
		OutPath.Append(Builder);
		if (OutValue)
		{
			*OutValue = Result;
		}
		return true;
	}
	else
	{
		return false;
	}
}

template <typename ValueType>
inline bool TDirectoryTree<ValueType>::TryFindClosestPath(FStringView Path, FStringBuilderBase& OutPath, const ValueType** OutValue) const
{
	ValueType* ResultValue;
	bool bResult = const_cast<TDirectoryTree*>(this)->TryFindClosestPath(Path, OutPath, &ResultValue);
	if (bResult)
	{
		if (OutValue)
		{
			*OutValue = ResultValue;
		}
	}
	return bResult;
}

template <typename ValueType>
inline bool TDirectoryTree<ValueType>::TryFindClosestPath(FStringView Path, FStringBuilderBase& OutPath, ValueType** OutValue)
{
	ValueType* Result = TryFindClosestPathInternal(Path, &OutPath);
	if (Result)
	{
		if (OutValue)
		{
			*OutValue = Result;
		}
		return true;
	}
	else
	{
		return false;
	}
}

template <typename ValueType>
inline ValueType* TDirectoryTree<ValueType>::TryFindClosestPathInternal(FStringView Path,
	FStringBuilderBase* OutPath)
{
	if (OutPath)
	{
		// We build the path as we go along, before we know whether we will return true,
		// so we have to reset it now
		OutPath->Reset();
	}
	if (!Path.IsEmpty())
	{
		TStringBuilder<16> NormalizeBuffer;
		NormalizePathForReading(Path, NormalizeBuffer);

		ValueType* Result = Root.TryFindClosestPath(Path, OutPath, PathSeparator);
		if (Result)
		{
			return Result;
		}
	}

	return Root.HasValue() ? &Root.GetValue() : nullptr;
}

template <typename ValueType>
inline bool TDirectoryTree<ValueType>::NormalizePathForReading(FStringView& Path,
	FStringBuilderBase& NormalizeBuffer) const
{
	// Drive specifiers without a root are a special case; they break our assumption that if
	// if	FPathViews::IsParentPathOf(DriveSpecifier, PathInThatDrive)
	// then DriveSpecifier == FirstComponentOfPathInThatDrive.
	// 'D:' is a parent path of 'D:/Path' but FirstComponent of 'D:/Path' is 'D:/' != 'D:' 
	// 
	// In general usage on e.g. Windows, drive specifiers without a path are interpreted to mean the current
	// working directory of the given drive. But we don't have that context so that meaning is not applicable.
	// 
	// We therefore instead interpret them to mean the root of the drive. Append the PathSeparator to make
	// them the root.
	if (FPathViews::IsDriveSpecifierWithoutRoot(Path))
	{
		FStringView Volume;
		FStringView Remainder;
		FPathViews::SplitVolumeSpecifier(Path, Volume, Remainder);
		NormalizeBuffer << Volume << PathSeparator << Remainder;
		Path = NormalizeBuffer.ToView();
		return true;
	}
	return false;
}

template <typename ValueType>
inline bool TDirectoryTree<ValueType>::NormalizePathForWriting(FStringView& Path, FStringBuilderBase& NormalizeBuffer)
{
	if (NormalizePathForReading(Path, NormalizeBuffer))
	{
		// If the call to NormalizePath came from a function that is adding paths to the tree, and
		// !bPathSeparatorInitialized, then leave a marker that we might need to fix up the added paths
		// when we encounter the user's desired path separator. 
		bNeedDriveWithoutPathFixup = !bPathSeparatorInitialized;
		return true;
	}
	return false;
}

template <typename ValueType>
inline void TDirectoryTree<ValueType>::InitializePathSeparator(TCHAR InPathSeparator)
{
	check(!bPathSeparatorInitialized);

	// If the requested PathSeparator is not the one we guessed it was when we had to
	// normalize a drive without a path (e.g. 'D:' -> 'D:/'), then fixup all those
	// drive children to have the desired separator.
	bNeedDriveWithoutPathFixup &= (InPathSeparator != PathSeparator);
	if (bNeedDriveWithoutPathFixup)
	{
		// The drives without paths will be direct children of the root so we only need
		// to fixup direct children
		Root.FixupDirectChildrenPathSeparator(PathSeparator, InPathSeparator);
		bNeedDriveWithoutPathFixup = false;
	}

	PathSeparator = InPathSeparator;
	bPathSeparatorInitialized = true;
}

template<typename ValueType>
inline bool TDirectoryTree<ValueType>::ContainsChildPaths(FStringView Path) const
{
	TStringBuilder<16> NormalizeBuffer;
	NormalizePathForReading(Path, NormalizeBuffer);

	return Root.ContainsChildPaths(Path);
}

template<typename ValueType>
inline bool TDirectoryTree<ValueType>::TryGetChildren(FStringView Path, TArray<FString>& OutRelativeChildNames,
	EDirectoryTreeGetFlags Flags) const
{
	if (Path.IsEmpty() && !EnumHasAnyFlags(Flags, EDirectoryTreeGetFlags::ImpliedParent) && !Root.HasValue())
	{
		return false;
	}
	TStringBuilder<16> NormalizeBuffer;
	NormalizePathForReading(Path, NormalizeBuffer);

	TStringBuilder<1024> ReportedPathPrefix;
	return Root.TryGetChildren(ReportedPathPrefix, PathSeparator, Path, OutRelativeChildNames, Flags);
}

template<typename ValueType>
TDirectoryTree<ValueType>::FTreeNode::FTreeNode(const FTreeNode& Other)
{
	*this = Other;
}

template<typename ValueType>
typename TDirectoryTree<ValueType>::FTreeNode& TDirectoryTree<ValueType>::FTreeNode::operator=(const FTreeNode& Other)
{
	Reset();

	if (Other.HasValue())
	{
		SetValue(CopyTemp(Other.GetValue()));
	}

	Realloc(Other.CapacityChildNodes);
	int32 NumChildren = Other.GetNumChildNodes();
	for (int32 i = 0; i < NumChildren; ++i)
	{
		RelPaths[i] = Other.RelPaths[i];
	}
	for (int32 i = 0; i < NumChildren; ++i)
	{
		ChildNodes[i] = Other.ChildNodes[i];
	}

	SetNumChildNodes(Other.GetNumChildNodes());
	return *this;
}

template <typename ValueType>
TDirectoryTree<ValueType>::FTreeNode::FTreeNode(FTreeNode&& Other)
{
	*this = MoveTemp(Other);
}

template <typename ValueType>
typename TDirectoryTree<ValueType>::FTreeNode& TDirectoryTree<ValueType>::FTreeNode::operator=(FTreeNode&& Other)
{
	if (Other.HasValue())
	{
		SetValue(MoveTemp(Other.GetValue()));
		Other.RemoveValue();
	}
	else
	{
		RemoveValue();
	}
	RelPaths = MoveTemp(Other.RelPaths);
	ChildNodes = MoveTemp(Other.ChildNodes);
	SetNumChildNodes(Other.GetNumChildNodes());
	CapacityChildNodes = MoveTemp(Other.CapacityChildNodes);

	Other.SetNumChildNodes(0);
	Other.CapacityChildNodes = 0;
	return *this;
}

template <typename ValueType>
void TDirectoryTree<ValueType>::FTreeNode::Reset()
{
	RemoveValue();
	RelPaths.Reset();
	ChildNodes.Reset();
	NumChildNodesAndFlags = 0;
	CapacityChildNodes = 0;
}

template <typename ValueType>
inline int32 TDirectoryTree<ValueType>::FTreeNode::GetNumChildNodes() const
{
	return (int32)(NumChildNodesAndFlags & ~FlagsMask);
}

template <typename ValueType>
inline void TDirectoryTree<ValueType>::FTreeNode::SetNumChildNodes(int32 InNumChildNodes)
{
	check(0 <= InNumChildNodes && (InNumChildNodes & FlagsMask) == 0);
	NumChildNodesAndFlags =
		InNumChildNodes |
		(NumChildNodesAndFlags & FlagsMask);
}

template <typename ValueType>
inline bool TDirectoryTree<ValueType>::FTreeNode::HasValue() const
{
	return (NumChildNodesAndFlags & FlagsMask) != 0;
}

template <typename ValueType>
inline ValueType& TDirectoryTree<ValueType>::FTreeNode::GetValue()
{
	return *reinterpret_cast<ValueType*>(&Value);
}

template<typename ValueType>
inline const ValueType& TDirectoryTree<ValueType>::FTreeNode::GetValue() const
{
	return *reinterpret_cast<const ValueType*>(&Value);
}

template <typename ValueType>
inline void TDirectoryTree<ValueType>::FTreeNode::SetValue(ValueType&& InValue)
{
	RemoveValue();

	uint32 FlagsValue = 0x1;
	NumChildNodesAndFlags =
		(NumChildNodesAndFlags & ~FlagsMask) |
		(FlagsValue << FlagsShift);
	new (&Value) ValueType(MoveTemp(InValue));
}

template <typename ValueType>
inline void TDirectoryTree<ValueType>::FTreeNode::SetDefaultValue()
{
	RemoveValue();

	uint32 FlagsValue = 0x1;
	NumChildNodesAndFlags =
		(NumChildNodesAndFlags & ~FlagsMask) |
		(FlagsValue << FlagsShift);
	new (&Value) ValueType();
}

template <typename ValueType>
inline void TDirectoryTree<ValueType>::FTreeNode::RemoveValue()
{
	if (HasValue())
	{
		uint32 FlagsValue = 0x0;
		NumChildNodesAndFlags =
			(NumChildNodesAndFlags & ~FlagsMask) |
			(FlagsValue << FlagsShift);

		// We need a typedef here because VC won't compile the destructor call below if ValueType itself has a member called ValueType
		typedef ValueType FDirectoryTreeDestructValueType;
		((ValueType*)&Value)->FDirectoryTreeDestructValueType::~FDirectoryTreeDestructValueType();
	}
}

template <typename ValueType>
ValueType& TDirectoryTree<ValueType>::FTreeNode::FindOrAdd(FStringView InRelPath, bool& bOutExisted)
{
	FStringView FirstComponent;
	FStringView RemainingPath;
	FPathViews::SplitFirstComponent(InRelPath, FirstComponent, RemainingPath);
	check(!FirstComponent.IsEmpty());
	bool bExists;
	int32 InsertionIndex = FindInsertionIndex(FirstComponent, bExists);
	if (!bExists)
	{
		bOutExisted = false;
		FTreeNode& ChildNode = InsertChildNode(InsertionIndex, FString(InRelPath), FTreeNode());
		ChildNode.SetDefaultValue();
		return ChildNode.GetValue();
	}


	FTreeNode& ChildNode = ChildNodes[InsertionIndex];
	FString& ChildRelPath = RelPaths[InsertionIndex];

	FStringView ExistingFirstComponent;
	FStringView ExistingRemainingPath;
	FPathViews::SplitFirstComponent(ChildRelPath, ExistingFirstComponent, ExistingRemainingPath);
	check(!ExistingFirstComponent.IsEmpty());
	int32 NumFirstComponents = 1;
	for (int32 RunawayLoop = 0; RunawayLoop <= InRelPath.Len(); ++RunawayLoop)
	{
		if (ExistingRemainingPath.IsEmpty())
		{
			// We've reached the end of the existing path
			if (!RemainingPath.IsEmpty())
			{
				// We have not reached the end of the input path, so it is a child of the existing path
				return ChildNode.FindOrAdd(RemainingPath, bOutExisted);
			}
			else
			{
				// The input path matches the existing path
				bOutExisted = ChildNode.HasValue();
				if (!ChildNode.HasValue())
				{
					ChildNode.SetDefaultValue();
				}
				return ChildNode.GetValue();
			}
		}
		else if (RemainingPath.IsEmpty())
		{
			// We've reached the end of the input path, so it is a parent of the existing path
			bOutExisted = false;

			// Create a new ChildNode and move the existing ChildNode into a child of it
			// We can modify RelPath in place without messing up the sort order because the nodes are sorted by
			// FirstComponent only and the FirstComponent is not changing
			FTreeNode OldTreeNode = MoveTemp(ChildNode);
			FString OldRelPath(ExistingRemainingPath);
			ChildNode.Reset();
			ChildNode.SetDefaultValue();
			ChildRelPath = InRelPath;
			ChildNode.InsertChildNode(0, MoveTemp(OldRelPath), MoveTemp(OldTreeNode));
			return ChildNode.GetValue();
		}
		else
		{
			// Both existing and remaining have more directory components
			FStringView NextFirstComponent;
			FStringView NextRemainingPath;
			FPathViews::SplitFirstComponent(RemainingPath, NextFirstComponent, NextRemainingPath);
			check(!NextFirstComponent.IsEmpty());
			FStringView NextExistingFirstComponent;
			FStringView NextExistingRemainingPath;
			FPathViews::SplitFirstComponent(ExistingRemainingPath, NextExistingFirstComponent, NextExistingRemainingPath);
			check(!NextExistingFirstComponent.IsEmpty());
			if (NextFirstComponent.Equals(NextExistingFirstComponent, ESearchCase::IgnoreCase)) // This works around a compiler bug. Should be changed to "NextFirstComponent == NextExistingFirstComponent"
			{
				// Next component is also a match, go to the next loop iteration to handle the new remainingpaths
				RemainingPath = NextRemainingPath;
				ExistingRemainingPath = NextExistingRemainingPath;
				++NumFirstComponents;
				continue;
			}
			else
			{
				// Existing and remaining firstcomponent differ, so they are both child paths of a mutual parent path
				// Reconstruct the CommonParentPath
				TStringBuilder<1024> CommonParentPath;
				FStringView ParentRemaining = ChildRelPath;
				for (int32 FirstComponentIndex = 0; FirstComponentIndex < NumFirstComponents; ++FirstComponentIndex)
				{
					FStringView ParentFirstComponent;
					FStringView NextParentRemaining;
					FPathViews::SplitFirstComponent(ParentRemaining, ParentFirstComponent, NextParentRemaining);
					FPathViews::Append(CommonParentPath, ParentFirstComponent);
					ParentRemaining = NextParentRemaining;
				}

				// Create a new ChildNode and move the existing ChildNode into a child of it
				// We can modify RelPath in place without messing up the sort order because the nodes are sorted by
				// FirstComponent only and the FirstComponent is not changing
				FTreeNode OldTreeNode = MoveTemp(ChildNode);
				FString OldRelPath(ExistingRemainingPath);
				ChildNode.Reset();
				ChildRelPath = CommonParentPath;
				ChildNode.InsertChildNode(0, MoveTemp(OldRelPath), MoveTemp(OldTreeNode));

				// The input path is now a child of the modified ChildNode
				// It's impossible for FindOrAdd to fail.
				// If it ever becomes possible for it to fail, we will have to remove the child we added if it fails.
				return ChildNode.FindOrAdd(RemainingPath, bOutExisted);
			}
		}
	}
	checkf(false, TEXT("Infinite loop trying to split path %.*s into components."), InRelPath.Len(), InRelPath.GetData());
	return GetValue();
}

template <typename ValueType>
void TDirectoryTree<ValueType>::FTreeNode::Remove(FStringView InRelPath, bool& bOutExisted)
{
	check(!InRelPath.IsEmpty());

	FStringView FirstComponent;
	FStringView RemainingPath;
	FPathViews::SplitFirstComponent(InRelPath, FirstComponent, RemainingPath);
	bool bExists;
	int32 InsertionIndex = FindInsertionIndex(FirstComponent, bExists);
	if (!bExists)
	{
		bOutExisted = false;
		return;
	}

	FTreeNode& ChildNode = ChildNodes[InsertionIndex];
	FString& ChildRelPath = RelPaths[InsertionIndex];

	if (!FPathViews::TryMakeChildPathRelativeTo(InRelPath, ChildRelPath, RemainingPath))
	{
		bOutExisted = false;
		return;
	}

	if (!RemainingPath.IsEmpty())
	{
		// The input path is a child of the existing path
		ChildNode.Remove(RemainingPath, bOutExisted);
		if (bOutExisted)
		{
			// If the remove was successful, the childnode must have had at least two paths, because
			// otherwise we would have previously Compacted it to combine its RelPath with RemainingPath.
			// In case it only had two paths and now has one path, try to compact it.
			ConditionalCompactNode(ChildRelPath, ChildNode);
		}
	}
	else
	{
		// The input path matches the existing path
		bOutExisted = ChildNode.HasValue();
		if (ChildNode.GetNumChildNodes() != 0)
		{
			if (ChildNode.HasValue())
			{
				ChildNode.RemoveValue();
				ConditionalCompactNode(ChildRelPath, ChildNode);
			}
		}
		else
		{
			RemoveChildNodeAt(InsertionIndex);
		}
	}
}

template <typename ValueType>
ValueType* TDirectoryTree<ValueType>::FTreeNode::Find(FStringView InRelPath)
{
	check(!InRelPath.IsEmpty());

	FStringView FirstComponent;
	FStringView RemainingPath;
	FPathViews::SplitFirstComponent(InRelPath, FirstComponent, RemainingPath);
	bool bExists;
	int32 InsertionIndex = FindInsertionIndex(FirstComponent, bExists);
	if (!bExists)
	{
		return nullptr;
	}

	FTreeNode& ChildNode = ChildNodes[InsertionIndex];
	FString& ChildRelPath = RelPaths[InsertionIndex];

	if (!FPathViews::TryMakeChildPathRelativeTo(InRelPath, ChildRelPath, RemainingPath))
	{
		return nullptr;
	}

	if (!RemainingPath.IsEmpty())
	{
		// The input path is a child of the existing path
		return ChildNode.Find(RemainingPath);
	}
	else
	{
		// The input path matches the existing path
		return ChildNode.HasValue() ? &ChildNode.GetValue() : nullptr;
	}
}

template<typename ValueType>
bool TDirectoryTree<ValueType>::FTreeNode::ContainsChildPaths(FStringView InRelPath) const
{
	if (InRelPath.IsEmpty())
	{
		// This is the node we were searching for as an explicit entry
		return GetNumChildNodes() > 0;
	}

	// We are still looking for the requested InRelPath and are not reporting results yet. Look for an existing
	// stored child that has the same FirstComponent of its path as InRelPath does.
	FStringView FirstComponent;
	FStringView RemainingPath;
	FPathViews::SplitFirstComponent(InRelPath, FirstComponent, RemainingPath);
	bool bExists;
	int32 InsertionIndex = FindInsertionIndex(FirstComponent, bExists);
	if (!bExists)
	{
		// No child has the same FirstComponent as InRelPath, so InRelPath does not exist in the tree, not even as
		// an implied path.
		return false;
	}

	FTreeNode& ChildNode = ChildNodes[InsertionIndex];
	FString& ChildRelPath = RelPaths[InsertionIndex];

	FStringView ExistingFirstComponent;
	FStringView ExistingRemainingPath;
	FPathViews::SplitFirstComponent(ChildRelPath, ExistingFirstComponent, ExistingRemainingPath);
	check(FPathViews::Equals(
		FirstComponent,
		ExistingFirstComponent)); // Otherwise FindInsertionIndex would have returned bExists=false

	// Same logic as TryGetChildren - progressively match the InRelPath against ChildRelPath until one is empty
	for (int32 RunawayLoop = 0; RunawayLoop <= InRelPath.Len(); ++RunawayLoop)
	{
		if (ExistingRemainingPath.IsEmpty())
		{
			// We've reached the end of the existing path, so InRelPath is either equal to or a child of the
			// ChildNode. Delegate to that node to keep searching or return results
			return ChildNode.ContainsChildPaths(RemainingPath);
		}
		else if (RemainingPath.IsEmpty())
		{
			// We've reached the end of the input path, but not the end of the existing path, so InRelPath is a
			// parent of the existing path, and is an implied path rather than an added path.
			return true;
		}
		else
		{
			// Both existing and remaining have more directory components
			FStringView NextFirstComponent;
			FStringView NextRemainingPath;
			FPathViews::SplitFirstComponent(RemainingPath, NextFirstComponent, NextRemainingPath);
			check(!NextFirstComponent.IsEmpty());
			FStringView NextExistingFirstComponent;
			FStringView NextExistingRemainingPath;
			FPathViews::SplitFirstComponent(ExistingRemainingPath, NextExistingFirstComponent, NextExistingRemainingPath);
			check(!NextExistingFirstComponent.IsEmpty());
			if (NextFirstComponent == NextExistingFirstComponent)
			{
				// Next component is also a match, go to the next loop iteration to handle the new remainingpaths
				RemainingPath = NextRemainingPath;
				ExistingRemainingPath = NextExistingRemainingPath;

				continue;
			}
			else
			{
				// The existing child diverges from the path components of InRelPath, so InRelPath does not exist
				// in the tree, not even as an implied path.
				return false;
			}
		}
	}
	checkf(false, TEXT("Infinite loop trying to split path %.*s into components."), InRelPath.Len(), InRelPath.GetData());
	return false;
}

template <typename ValueType>
bool TDirectoryTree<ValueType>::FTreeNode::TryGetChildren(FStringBuilderBase& ReportPathPrefix,
	TCHAR InPathSeparator, FStringView InRelPath, TArray<FString>& OutRelativeChildNames,
	EDirectoryTreeGetFlags Flags) const
{
	if (InRelPath.IsEmpty())
	{
		// RelPath indicates this node, so append the children of this node to the list.
		// Caller is responsible for not calling TryGetChildren on *this if results for *this
		// should not be returned due to EDirectoryTreeGetFlags.

		int32 NumChildNodes = GetNumChildNodes();
		for (int32 Index = 0; Index < NumChildNodes; ++Index)
		{
			const FTreeNode& ChildNode = ChildNodes[Index];
			const FString& ChildRelPath = RelPaths[Index];

			if (EnumHasAnyFlags(Flags, EDirectoryTreeGetFlags::ImpliedChildren))
			{
				// When ImpliedChildren are supposed to be reported, iterate over every stored child
				// and report the first component of its RelPath as a child.
				// If recursive, also report the remaining components of its RelPath, and then
				// forward the call to it to return its recursive children.
				FStringView FirstComponent;
				FStringView RemainingPath;
				FPathViews::SplitFirstComponent(ChildRelPath, FirstComponent, RemainingPath);

				int32 SavedLen = ReportPathPrefix.Len();
				FPathViews::Append(ReportPathPrefix, FirstComponent);
				UE::DirectoryTree::FixupPathSeparator(ReportPathPrefix, SavedLen, InPathSeparator);
				OutRelativeChildNames.Add(*ReportPathPrefix);

				if (EnumHasAnyFlags(Flags, EDirectoryTreeGetFlags::Recursive))
				{
					while (!RemainingPath.IsEmpty())
					{
						FPathViews::SplitFirstComponent(RemainingPath, FirstComponent, RemainingPath);

						int32 SavedLenForSubPath = ReportPathPrefix.Len();
						FPathViews::Append(ReportPathPrefix, FirstComponent);
						UE::DirectoryTree::FixupPathSeparator(ReportPathPrefix, SavedLenForSubPath, InPathSeparator);
						OutRelativeChildNames.Add(*ReportPathPrefix);
					}

					(void)ChildNode.TryGetChildren(ReportPathPrefix, InPathSeparator, FStringView(),
						OutRelativeChildNames, Flags);
				}
				ReportPathPrefix.RemoveSuffix(ReportPathPrefix.Len() - SavedLen);
			}
			else
			{
				// When ImpliedChildren are not supposed to be reported, report each stored child by its full relpath,
				// unless the child is an implied path. If the child is an implied path, recursively ask the child
				// to report its added children. Also, if user requested recursive, ask the child to return its
				// recursive children even if it has a value.
				int32 SavedLen = ReportPathPrefix.Len();
				FPathViews::Append(ReportPathPrefix, ChildRelPath);
				UE::DirectoryTree::FixupPathSeparator(ReportPathPrefix, SavedLen, InPathSeparator);

				if (ChildNode.HasValue())
				{
					OutRelativeChildNames.Add(*ReportPathPrefix);
				}
				if (!ChildNode.HasValue() || EnumHasAnyFlags(Flags, EDirectoryTreeGetFlags::Recursive))
				{
					(void)ChildNode.TryGetChildren(ReportPathPrefix, InPathSeparator, FStringView(),
						OutRelativeChildNames, Flags);
				}
				ReportPathPrefix.RemoveSuffix(ReportPathPrefix.Len() - SavedLen);
			}
		}

		return true;
	}
	else
	{
		// We are still looking for the requested InRelPath and are not reporting results yet. Look for an existing
		// stored child that has the same FirstComponent of its path as InRelPath does.
		FStringView FirstComponent;
		FStringView RemainingPath;
		FPathViews::SplitFirstComponent(InRelPath, FirstComponent, RemainingPath);
		bool bExists;
		int32 InsertionIndex = FindInsertionIndex(FirstComponent, bExists);
		if (!bExists)
		{
			// No child has the same FirstComponent as InRelPath, so InRelPath does not exist in the tree, not even as
			// an implied path.
			return false;
		}

		FTreeNode& ChildNode = ChildNodes[InsertionIndex];
		FString& ChildRelPath = RelPaths[InsertionIndex];

		FStringView ExistingFirstComponent;
		FStringView ExistingRemainingPath;
		FPathViews::SplitFirstComponent(ChildRelPath, ExistingFirstComponent, ExistingRemainingPath);
		check(FPathViews::Equals(FirstComponent, ExistingFirstComponent)); // Otherwise FindInsertionIndex would have returned bExists=false

		for (int32 RunawayLoop = 0; RunawayLoop <= InRelPath.Len(); ++RunawayLoop)
		{
			if (ExistingRemainingPath.IsEmpty())
			{
				// We've reached the end of the existing path, so InRelPath is either equal to or a child of the
				// ChildNode. If it is equal to the ChildNode, it is our responsibility to NOT call TryGetChildren
				// if the ChildNode is an implied path and ImpliedParent flag is not requested.
				if (RemainingPath.IsEmpty() && !EnumHasAnyFlags(Flags, EDirectoryTreeGetFlags::ImpliedParent) &&
					!ChildNode.HasValue())
				{
					return false;
				}

				return ChildNode.TryGetChildren(ReportPathPrefix, InPathSeparator, RemainingPath,
					OutRelativeChildNames, Flags);
			}
			else if (RemainingPath.IsEmpty())
			{
				// We've reached the end of the input path, but not the end of the existing path, so InRelPath is a
				// parent of the existing path, and is an implied path rather than an added path.
				if (!EnumHasAnyFlags(Flags, EDirectoryTreeGetFlags::ImpliedParent))
				{
					return false;
				}

				if (EnumHasAnyFlags(Flags, EDirectoryTreeGetFlags::ImpliedChildren))
				{
					// When implied children are supposed to be reported, add the next pathcomponent of the remaining
					// child path as the first reported child of InRelPath. If recursive, also add all the remaining
					// components of the existing child path and then forward on to the child path for all of its children.
					FPathViews::SplitFirstComponent(ExistingRemainingPath, ExistingFirstComponent,
						ExistingRemainingPath);
					int32 SavedLen = ReportPathPrefix.Len();
					FPathViews::Append(ReportPathPrefix, ExistingFirstComponent);
					UE::DirectoryTree::FixupPathSeparator(ReportPathPrefix, SavedLen, InPathSeparator);
					OutRelativeChildNames.Add(*ReportPathPrefix);

					if (EnumHasAnyFlags(Flags, EDirectoryTreeGetFlags::Recursive))
					{
						while (!ExistingRemainingPath.IsEmpty())
						{
							FPathViews::SplitFirstComponent(ExistingRemainingPath, ExistingFirstComponent,
								ExistingRemainingPath);

							int32 SavedLenForSubPath = ReportPathPrefix.Len();
							FPathViews::Append(ReportPathPrefix, ExistingFirstComponent);
							UE::DirectoryTree::FixupPathSeparator(ReportPathPrefix, SavedLenForSubPath
								, InPathSeparator);
							OutRelativeChildNames.Add(*ReportPathPrefix);
						}

						(void) ChildNode.TryGetChildren(ReportPathPrefix, InPathSeparator, FStringView(),
							OutRelativeChildNames, Flags);
					}

					ReportPathPrefix.RemoveSuffix(ReportPathPrefix.Len() - SavedLen);
				}
				else
				{
					// When implied children are not supposed to be reported, report the remaining components of the
					// existing child path as a single string as the first reported child of InRelPath, but only if
					// it is not an implied path. If recursive, forward to the childnode for all of its children.
					if (ChildNode.HasValue() || EnumHasAnyFlags(Flags, EDirectoryTreeGetFlags::Recursive))
					{
						int32 SavedLenForSubPath = ReportPathPrefix.Len();
						FPathViews::Append(ReportPathPrefix, ExistingRemainingPath);
						UE::DirectoryTree::FixupPathSeparator(ReportPathPrefix, SavedLenForSubPath, InPathSeparator);

						if (ChildNode.HasValue())
						{
							OutRelativeChildNames.Add(*ReportPathPrefix);
						}
						if (EnumHasAnyFlags(Flags, EDirectoryTreeGetFlags::Recursive))
						{
							(void)ChildNode.TryGetChildren(ReportPathPrefix, InPathSeparator, FStringView(),
								OutRelativeChildNames, Flags);
						}
					}
				}

				return true;
			}
			else
			{
				// Both existing and remaining have more directory components
				FStringView NextFirstComponent;
				FStringView NextRemainingPath;
				FPathViews::SplitFirstComponent(RemainingPath, NextFirstComponent, NextRemainingPath);
				check(!NextFirstComponent.IsEmpty());
				FStringView NextExistingFirstComponent;
				FStringView NextExistingRemainingPath;
				FPathViews::SplitFirstComponent(ExistingRemainingPath, NextExistingFirstComponent, NextExistingRemainingPath);
				check(!NextExistingFirstComponent.IsEmpty());
				if (NextFirstComponent == NextExistingFirstComponent)
				{
					// Next component is also a match, go to the next loop iteration to handle the new remainingpaths
					RemainingPath = NextRemainingPath;
					ExistingRemainingPath = NextExistingRemainingPath;

					continue;
				}
				else
				{
					// The existing child diverges from the path components of InRelPath, so InRelPath does not exist
					// in the tree, not even as an implied path.
					return false;
				}
			}
		}
		checkf(false, TEXT("Infinite loop trying to split path %.*s into components."), InRelPath.Len(), InRelPath.GetData());
	}

	check(false); //-V779 Only way to get here is from the checkf in the else block.
	return false;
}

template <typename ValueType>
ValueType* TDirectoryTree<ValueType>::FTreeNode::TryFindClosestPath(FStringView InRelPath,
	FStringBuilderBase* OutPath, TCHAR InPathSeparator)
{
	check(!InRelPath.IsEmpty());

	FStringView FirstComponent;
	FStringView RemainingPath;
	FPathViews::SplitFirstComponent(InRelPath, FirstComponent, RemainingPath);
	bool bExists;
	int32 InsertionIndex = FindInsertionIndex(FirstComponent, bExists);
	if (!bExists)
	{
		return nullptr;
	}

	FTreeNode& ChildNode = ChildNodes[InsertionIndex];
	const FString& ChildRelPath = RelPaths[InsertionIndex];

	if (!FPathViews::TryMakeChildPathRelativeTo(InRelPath, ChildRelPath, RemainingPath))
	{
		return nullptr;
	}

	if (!RemainingPath.IsEmpty())
	{
		// The input path is a child of the existing path
		int32 SavedOutPathLen = INDEX_NONE;
		if (OutPath)
		{
			SavedOutPathLen = OutPath->Len();
			FPathViews::Append(*OutPath, ChildRelPath);
			UE::DirectoryTree::FixupPathSeparator(*OutPath, SavedOutPathLen, InPathSeparator);
		}
		ValueType* Result = ChildNode.TryFindClosestPath(RemainingPath, OutPath, InPathSeparator);
		if (Result)
		{
			return Result;
		}

		if (ChildNode.HasValue())
		{
			return &ChildNode.GetValue();
		}
		if (OutPath)
		{
			OutPath->RemoveSuffix(OutPath->Len() - SavedOutPathLen);
		}
		return nullptr;
	}
	else
	{
		// The input path matches the existing path
		if (!ChildNode.HasValue())
		{
			return nullptr;
		}
		if (OutPath)
		{
			int32 SavedOutPathLen = OutPath->Len();
			FPathViews::Append(*OutPath, ChildRelPath);
			UE::DirectoryTree::FixupPathSeparator(*OutPath, SavedOutPathLen, InPathSeparator);
		}
		return &ChildNode.GetValue();
	}
}

template <typename ValueType>
inline bool TDirectoryTree<ValueType>::FTreeNode::IsEmpty() const
{
	return !HasValue() && GetNumChildNodes() == 0;
}

template <typename ValueType>
inline SIZE_T TDirectoryTree<ValueType>::FTreeNode::GetAllocatedSize() const
{
	SIZE_T Size = 0;
	Size = CapacityChildNodes * (sizeof(FTreeNode) + sizeof(FString));
	int32 NumChildNodes = GetNumChildNodes();
	for (const FString& RelPath : TConstArrayView<FString>(RelPaths.Get(), NumChildNodes))
	{
		Size += RelPath.GetAllocatedSize();
	}
	for (const FTreeNode& ChildNode : TConstArrayView<FTreeNode>(ChildNodes.Get(), NumChildNodes))
	{
		Size += ChildNode.GetAllocatedSize();
	}
	return Size;
}

template <typename ValueType>
inline void TDirectoryTree<ValueType>::FTreeNode::Shrink()
{
	int32 NumChildNodes = GetNumChildNodes();
	Realloc(NumChildNodes);

	for (FString& RelPath : TArrayView<FString>(RelPaths.Get(), NumChildNodes))
	{
		RelPath.Shrink();
	}
	for (FTreeNode& ChildNode : TArrayView<FTreeNode>(ChildNodes.Get(), NumChildNodes))
	{
		ChildNode.Shrink();
	}
}

template <typename ValueType>
inline int32 TDirectoryTree<ValueType>::FTreeNode::FindInsertionIndex(FStringView FirstPathComponent, bool& bOutExists) const
{
	return UE::DirectoryTree::FindInsertionIndex(GetNumChildNodes(), RelPaths, FirstPathComponent, bOutExists);
}

template <typename ValueType>
typename TDirectoryTree<ValueType>::FTreeNode& TDirectoryTree<ValueType>::FTreeNode::InsertChildNode(int32 InsertionIndex, FString&& RelPath,
	FTreeNode&& ChildNode)
{
	int32 NumChildNodes = GetNumChildNodes();
	check(0 <= InsertionIndex && InsertionIndex <= NumChildNodes);
	checkf((((uint32)(NumChildNodes + 1)) & FlagsMask) == 0, TEXT("Overflow"));

	if (NumChildNodes == CapacityChildNodes)
	{
		int32 NewCapacity;
		if (CapacityChildNodes == 0)
		{
			NewCapacity = 1;
		}
		else if (CapacityChildNodes < 8)
		{
			NewCapacity = CapacityChildNodes * 2;
		}
		else
		{
			NewCapacity = (CapacityChildNodes * 3) / 2;
			checkf(NewCapacity > CapacityChildNodes, TEXT("Overflow"));
		}
		Realloc(NewCapacity);
		check(NumChildNodes < CapacityChildNodes);
	}

	for (int32 ShiftIndex = NumChildNodes; ShiftIndex > InsertionIndex; --ShiftIndex)
	{
		RelPaths[ShiftIndex] = MoveTemp(RelPaths[ShiftIndex - 1]);
		ChildNodes[ShiftIndex] = MoveTemp(ChildNodes[ShiftIndex - 1]);
	}
	RelPaths[InsertionIndex] = MoveTemp(RelPath);
	ChildNodes[InsertionIndex] = MoveTemp(ChildNode);
	SetNumChildNodes(NumChildNodes + 1);
	return ChildNodes[InsertionIndex];
}

template <typename ValueType>
void TDirectoryTree<ValueType>::FTreeNode::RemoveChildNodeAt(int32 RemoveIndex)
{
	int32 NumChildNodes = GetNumChildNodes();
	check(0 <= RemoveIndex && RemoveIndex < NumChildNodes);
	for (int32 ShiftIndex = RemoveIndex; ShiftIndex < NumChildNodes - 1; ++ShiftIndex)
	{
		RelPaths[ShiftIndex] = MoveTemp(RelPaths[ShiftIndex + 1]);
		ChildNodes[ShiftIndex] = MoveTemp(ChildNodes[ShiftIndex + 1]);
	}
	RelPaths[NumChildNodes - 1].Empty();
	ChildNodes[NumChildNodes - 1].Reset();
	SetNumChildNodes(NumChildNodes - 1);
}

template <typename ValueType>
void TDirectoryTree<ValueType>::FTreeNode::ConditionalCompactNode(FString& RelPath, FTreeNode& ChildNode)
{
	if (ChildNode.HasValue())
	{
		return;
	}

	int32 NumChildNodes = ChildNode.GetNumChildNodes();
	checkf(NumChildNodes > 0, TEXT("Invalid to call ConditionalCompactNode with an empty ChildNode."));
	if (NumChildNodes > 1)
	{
		return;
	}
	TStringBuilder<1024> NewRelPath;
	FPathViews::AppendPath(NewRelPath, RelPath);
	FPathViews::AppendPath(NewRelPath, ChildNode.RelPaths[0]);
	RelPath = NewRelPath;
	FTreeNode OldGrandChild = MoveTemp(ChildNode.ChildNodes[0]);
	ChildNode = MoveTemp(OldGrandChild);
}

template <typename ValueType>
void TDirectoryTree<ValueType>::FTreeNode::Realloc(int32 NewCapacity)
{
	if (CapacityChildNodes == NewCapacity)
	{
		return;
	}
	int32 NumChildNodes = GetNumChildNodes();
	check(NumChildNodes <= NewCapacity);
	if (NewCapacity > 0)
	{
		FString* NewRelPaths = new FString[NewCapacity];
		FTreeNode* NewChildNodes = new FTreeNode[NewCapacity];
		for (int32 Index = 0; Index < NumChildNodes; ++Index)
		{
			// Index < NumChildNodes <= NewCapacity. Some analyzers miss check(NumChildNodes <= NewCapacity) and need a direct hint.
			CA_ASSUME(Index < NewCapacity);
			NewRelPaths[Index] = MoveTemp(RelPaths[Index]);
			NewChildNodes[Index] = MoveTemp(ChildNodes[Index]);
		}
		ChildNodes.Reset(NewChildNodes);
		RelPaths.Reset(NewRelPaths);
	}
	else
	{
		ChildNodes.Reset();
		RelPaths.Reset();
	}
	CapacityChildNodes = NewCapacity;
}

template <typename ValueType>
inline void TDirectoryTree<ValueType>::FTreeNode::FixupDirectChildrenPathSeparator(TCHAR OldSeparator,
	TCHAR NewSeparator)
{
	int32 Num = GetNumChildNodes();
	for (int32 Index = 0; Index < Num; ++Index)
	{
		RelPaths[Index].ReplaceCharInline(OldSeparator, NewSeparator);
	}
}

template <typename ValueType>
inline TDirectoryTree<ValueType>::FIteratorInternal::FIteratorInternal(FTreeNode& Root, EDirectoryTreeGetFlags InFlags, TCHAR InPathSeparator)
	: Flags(InFlags)
	, PathSeparator(InPathSeparator)
{
	FStackData& RootData = Stack.Emplace_GetRef(Root, nullptr /* ParentClosestValue */);
	// The root node always exists. It can have a value, if empty string was pushed into the tree. Otherwise it does
	// not have a value, and is not treated as an implied parent. The highest parent directory of any added path other
	// than empty string is a child node of the root node.
	if (!RootData.Node->HasValue())
	{
		// To allow our node-based iteration to work and skip recording a value for the root, we need to skip past the
		// "Node's value itself" state (ChildIndex == -1) for the root node, and start at ChildIndex=0.
		RootData.ChildIndex = 0;
	}
	TraverseToValid();
}

template <typename ValueType>
inline TPair<FStringView, ValueType*> TDirectoryTree<ValueType>::FIteratorInternal::operator*() const
{
	check(!Stack.IsEmpty());
	const FStackData& Data = Stack.Last();
	FTreeNode* Node = Data.Node;
	return TPair<FStringView, ValueType*>(Path, Data.ClosestValue);
}

template <typename ValueType>
inline TDirectoryTree<ValueType>::FIteratorInternal::operator bool() const
{
	return !Stack.IsEmpty();
}

template <typename ValueType>
inline bool TDirectoryTree<ValueType>::FIteratorInternal::operator!=(const FIterationSentinel&) const
{
	return (bool)*this;
}

template <typename ValueType>
typename TDirectoryTree<ValueType>::FIteratorInternal& TDirectoryTree<ValueType>::FIteratorInternal::operator++()
{
	check((bool)*this); // It is invalid to call operator++ on the iterator when it is done

	// Stack.Last is pointing at an existing value, move past it
	FStackData* Data = &Stack.Last();
	if (Data->ChildIndex == -1)
	{
		// Pointing to the Node's value itself, move to the first child.
		Data->ChildIndex = 0;
	}
	else
	{
		// Inbetween states are not possible for the top node in the stack. TraverseToValid moves past
		// inbetween states before returning.
		check(Data->ChildIndex < Data->Node->GetNumChildNodes());
		check(Data->bChildRelPathInitialized);
		// Empty RemainingChildRelPath is an inbetween state, with or without bChildNodeInitialized.
		// TraverseToValid moves past it to add the next child node to the stack before returning.
		check(!Data->RemainingChildRelPath.IsEmpty());

		// Pointing to somewhere in the iteration of directories in the childnode's relative path.
		IncrementRemainingChildRelPath(Data);
	}

	TraverseToValid();
	return *this;
}

template <typename ValueType>
void TDirectoryTree<ValueType>::FIteratorInternal::TraverseToValid()
{
	while (!Stack.IsEmpty())
	{
		FStackData* Data = &Stack.Last();
		for (;;)
		{
			if (Data->ChildIndex == -1)
			{
				// Pointing to the Node's value itself. Return if the node has a value or implied children are allowed
				if (EnumHasAnyFlags(Flags, EDirectoryTreeGetFlags::ImpliedChildren)
					|| Data->Node->HasValue())
				{
					return;
				}

				if (!EnumHasAnyFlags(Flags, EDirectoryTreeGetFlags::Recursive) && Stack.Num() > 1)
				{
					// In the non-recursive case, we are iterating over the node in Stack[0], and Stack[1]
					// is each child node that we need to report. Do not iterate over the children of Stack[1];
					// skip to the end of its children.
					Data->ChildIndex = Data->Node->GetNumChildNodes();
				}
				else
				{
					Data->ChildIndex = 0;
				}
			}
			else if (Data->ChildIndex < Data->Node->GetNumChildNodes())
			{
				if (!Data->bChildRelPathInitialized)
				{
					Data->bChildRelPathInitialized = true;
					Data->PathLenBeforeChildNode = Path.Len();

					// If we are not returning implied children then skip straight to the recursive child node
					if (!EnumHasAnyFlags(Flags, EDirectoryTreeGetFlags::ImpliedChildren))
					{
						Data->RemainingChildRelPath = FStringView();
						int32 FixupStart = Path.Len();
						FPathViews::Append(Path, Data->Node->RelPaths[Data->ChildIndex]);
						UE::DirectoryTree::FixupPathSeparator(Path, FixupStart, PathSeparator);
					}
					else
					{
						Data->RemainingChildRelPath = Data->Node->RelPaths[Data->ChildIndex];
						// Pull the first component of the child relpath into our path, and
						// continue the loop to check for whether it is an implied parent or
						// the child node itself.
						IncrementRemainingChildRelPath(Data);
					}
				}
				else if (!Data->RemainingChildRelPath.IsEmpty())
				{
					// Pointing to somewhere in the iteration of directories in the childnode's relative path.
					// Return it as an implied node.
					return;
				}
				else if (!Data->bChildNodeInitialized)
				{
					Data->bChildNodeInitialized = true;
					Stack.Add(FStackData(Data->Node->ChildNodes[Data->ChildIndex], Data->ClosestValue));
					break; // Return to the outer loop to traverse into the node we just pushed
				}
				else
				{
					// Pointing to the recursive child node that we just popped, move to the next.
					++Data->ChildIndex;

					Data->bChildRelPathInitialized = false;
					Data->bChildNodeInitialized = false;
					Data->RemainingChildRelPath = FStringView();
					Path.RemoveSuffix(Path.Len() - Data->PathLenBeforeChildNode);
					Data->PathLenBeforeChildNode = 0;
					// Continue on to the next child
				}
			}
			else
			{
				Stack.Pop(EAllowShrinking::No);
				break; // Return to the outer loop to return traversal to the parent node
			}
		}
	}
}

template <typename ValueType>
inline void TDirectoryTree<ValueType>::FIteratorInternal::IncrementRemainingChildRelPath(FStackData* Data)
{
	FStringView FirstComponent;
	FStringView RemainingPath;
	FPathViews::SplitFirstComponent(Data->RemainingChildRelPath, FirstComponent, RemainingPath);
	Data->RemainingChildRelPath = RemainingPath;

	int32 FixupStart = Path.Len();
	FPathViews::Append(Path, FirstComponent);
	UE::DirectoryTree::FixupPathSeparator(Path, FixupStart, PathSeparator);
}

template <typename ValueType>
inline TDirectoryTree<ValueType>::FIteratorInternal::FStackData::FStackData(
	FTreeNode& InNode, ValueType* ParentClosestValue)
	: Node(&InNode)
	, ClosestValue(InNode.HasValue() ? &InNode.GetValue() : ParentClosestValue)
	, ChildIndex(-1)
	, PathLenBeforeChildNode(0)
	, bChildRelPathInitialized(false)
	, bChildNodeInitialized(false)
{
}

template <typename ValueType>
template <typename ViewedValueType>
inline TDirectoryTree<ValueType>::TIterator<ViewedValueType>::TIterator(FTreeNode& Root,
	EDirectoryTreeGetFlags InFlags, TCHAR InPathSeparator)
	: Internal(Root, InFlags, InPathSeparator)
{
}

template <typename ValueType>
template <typename ViewedValueType>
inline TPair<FStringView, ViewedValueType&> TDirectoryTree<ValueType>::TIterator<ViewedValueType>::operator*() const
{
	check((bool)Internal);
	TPair<FStringView, ValueType*> InternalPair = *Internal;
	check(InternalPair.Value != nullptr);
	return TPair<FStringView, ViewedValueType&>(InternalPair.Key, *InternalPair.Value);
}

template <typename ValueType>
template <typename ViewedValueType>
inline TArrowWrapper<TPair<FStringView, ViewedValueType&>>
TDirectoryTree<ValueType>::TIterator<ViewedValueType>::operator->() const
{
	return TArrowWrapper<TPair<FStringView, ViewedValueType&>>(**this);
}

template <typename ValueType>
template <typename ViewedValueType>
inline bool TDirectoryTree<ValueType>::TIterator<ViewedValueType>::operator!=(const FIterationSentinel&) const
{
	return (bool)*this;
}

template <typename ValueType>
template <typename ViewedValueType>
inline typename TDirectoryTree<ValueType>::template TIterator<ViewedValueType>& 
TDirectoryTree<ValueType>::TIterator<ViewedValueType>::operator++()
{
	++Internal;
	return *this;
}

template <typename ValueType>
template <typename ViewedValueType>
inline TDirectoryTree<ValueType>::TIterator<ViewedValueType>::operator bool() const
{
	return (bool)Internal;
}

template <typename ValueType>
template <typename ViewedValueType>
inline TDirectoryTree<ValueType>::TPointerIterator<ViewedValueType>::TPointerIterator(FTreeNode& Root,
	EDirectoryTreeGetFlags InFlags, TCHAR InPathSeparator)
	: Internal(Root, InFlags, InPathSeparator)
{
}

template <typename ValueType>
template <typename ViewedValueType>
inline TPair<FStringView, ViewedValueType*>
TDirectoryTree<ValueType>::TPointerIterator<ViewedValueType>::operator*() const
{
	check((bool)Internal);
	TPair<FStringView, ValueType*> InternalPair = *Internal;
	return TPair<FStringView, ViewedValueType*>(InternalPair.Key, InternalPair.Value);
}

template <typename ValueType>
template <typename ViewedValueType>
inline TArrowWrapper<TPair<FStringView, ViewedValueType*>>
TDirectoryTree<ValueType>::TPointerIterator<ViewedValueType>::operator->() const
{
	return TArrowWrapper<TPair<FStringView, ViewedValueType*>>(**this);
}

template <typename ValueType>
template <typename ViewedValueType>
inline bool TDirectoryTree<ValueType>::TPointerIterator<ViewedValueType>::operator!=(const FIterationSentinel&) const
{
	return (bool)Internal;
}

template <typename ValueType>
template <typename ViewedValueType>
inline typename TDirectoryTree<ValueType>::template TPointerIterator<ViewedValueType>&
TDirectoryTree<ValueType>::TPointerIterator<ViewedValueType>::operator++()
{
	++Internal;
	return *this;
}

template <typename ValueType>
template <typename ViewedValueType>
inline TDirectoryTree<ValueType>::TPointerIterator<ViewedValueType>::operator bool() const
{
	return (bool)Internal;
}

template <typename ValueType>
inline typename TDirectoryTree<ValueType>::FIterator TDirectoryTree<ValueType>::CreateIterator()
{
	return FIterator(Root,
		EDirectoryTreeGetFlags::Recursive,
		PathSeparator);
}

template <typename ValueType>
inline typename TDirectoryTree<ValueType>::FConstIterator TDirectoryTree<ValueType>::CreateConstIterator() const
{
	return FConstIterator(const_cast<FTreeNode&>(Root),
		EDirectoryTreeGetFlags::Recursive,
		PathSeparator);
}

template <typename ValueType>
inline typename TDirectoryTree<ValueType>::FPointerIterator TDirectoryTree<ValueType>::CreateIteratorForImplied()
{
	return FPointerIterator(Root,
		EDirectoryTreeGetFlags::Recursive | EDirectoryTreeGetFlags::ImpliedChildren, 
		PathSeparator);
}

template <typename ValueType>
inline typename TDirectoryTree<ValueType>::FConstPointerIterator TDirectoryTree<ValueType>::CreateConstIteratorForImplied() const
{
	return FConstPointerIterator(const_cast<FTreeNode&>(Root),
		EDirectoryTreeGetFlags::Recursive | EDirectoryTreeGetFlags::ImpliedChildren,
		PathSeparator);
}

template <typename ValueType>
inline typename TDirectoryTree<ValueType>::FIterator TDirectoryTree<ValueType>::begin()
{
	return CreateIterator();
}

template <typename ValueType>
inline typename TDirectoryTree<ValueType>::FConstIterator TDirectoryTree<ValueType>::begin() const
{
	return CreateConstIterator();
}

template <typename ValueType>
inline typename TDirectoryTree<ValueType>::FIterationSentinel TDirectoryTree<ValueType>::end() const
{
	return FIterationSentinel();
}
