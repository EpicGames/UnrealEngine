// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/CriticalSection.h"

/*
Implements a thread-safe discardable Key/Value cache by using two maps - a primary and a backfill.

Find() checks the current map first then the backfill. Entries found in the backfill are moved into
the primary map.

When Swap is called all items in the backfill are removed and the currentmap & backfill are swapped.

This results in most recently used items being retained and older/unused items being evicted. The more often Swap
is called the shorter the lifespan for items
*/
template<class KeyType, class ValueType>
class TDiscardableKeyValueCache
{
public:

	/* Flags used when calling Find() */
	struct LockFlags
	{
		enum Flags
		{
			ReadLock = (1 << 0),
			WriteLock = (1 << 1),
			WriteLockOnAddFail = (1 << 2),
		};
	};

	typedef TMap<KeyType, ValueType> TypedMap;

	[[nodiscard]] TDiscardableKeyValueCache() :
		CurrentMap(&Map1)
		, BackfillMap(&Map2)
	{
	}

	/* Access to the internal locking object */
	[[nodiscard]] FRWLock&  RWLock()
	{
		return LockObject;
	}

	/* Reference to the current map */
	[[nodiscard]] TypedMap& Current()
	{
		return *CurrentMap;
	}

	/* Reference to the current map */
	[[nodiscard]] TypedMap& Backfill()
	{
		return *BackfillMap;
	}

	/* Returns the total number of items in the cache*/
	[[nodiscard]] int32 Num()
	{
		uint32 LockFlags = ApplyLock(0, LockFlags::ReadLock);

		int32 Count = Map1.Num() + Map2.Num();

		Unlock(LockFlags);

		return Count;
	}

	/**
	*  Returns true and sets OutType to the value with the associated key if it exists.
	*/
	[[nodiscard]] bool Find(const KeyType& Key, ValueType& OutType)
	{
		uint32 LockFlags = ApplyLock(0, LockFlags::ReadLock);

		bool Found = InternalFindWhileLocked(Key, OutType, LockFlags, LockFlags);

		Unlock(LockFlags);
		return Found;
	}

	/**
	 *  Externally-lock-aware Find function. 
	 *
	 *	InFlags represents the currently locked state of the object, OutFlags the state after the
	 *	find operation has completed. Caller should be sure to unlock this object with OutFlags
	 */
	[[nodiscard]] bool Find(const KeyType& Key, ValueType& OutType, uint32 InCurrentLockFlags, uint32& OutLockFlags)
	{
		return InternalFindWhileLocked(Key, OutType, InCurrentLockFlags, OutLockFlags);
	}

	/**
	* Add an entry to the current map. Can fail if another thread has inserted
	* a matching object, in which case another call to Find() should succeed
	*/
	bool Add(const KeyType& Key, const ValueType& Value)
	{
		uint32 LockFlags = ApplyLock(0, LockFlags::WriteLock);
		bool Success = Add(Key, Value, LockFlags);
		Unlock(LockFlags);
		return Success;
	}

	/**
	* Add an entry to the current map. Can fail if another thread has inserted
	* a matching object, in which case another call to Find() should succeed
	*/
	bool Add(const KeyType& Key, const ValueType& Value, const uint32 LockFlags)
	{
		bool Success = true;

		checkf((LockFlags & LockFlags::WriteLock) != 0, TEXT("Cache is not locked for write during Add!"));

		// key is already here! likely another thread may have filled the cache. calling code should handle this
		// or request that a write lock be left after a Find() fails.
		if (CurrentMap->Contains(Key) == false)
		{
			CurrentMap->Add(Key, Value);
		}
		else
		{
			Success = false;
		}

		return Success;
	}

	/* 
		Discard all items left in the backfill and swap the current & backfill pointers
	*/
	int32 Discard()
	{
		uint32 LockFlags = ApplyLock(0, LockFlags::WriteLock);
		
		int32 Discarded = Discard(LockFlags, LockFlags, [](ValueType& Type) {});

		Unlock(LockFlags);

		return Discarded;
	}

	template<typename DeleteFunc>
	int32 Discard(DeleteFunc Func)
	{
		uint32 LockFlags = ApplyLock(0, LockFlags::WriteLock);

		int32 Discarded = Discard(LockFlags, LockFlags, Func);

		Unlock(LockFlags);

		return Discarded;
	}

	/**
	 * Discard all items in the backfill and swap the current & backfill pointers
	 */
	template<typename DeleteFunc>
	int32 Discard(uint32 InCurrentLockFlags, uint32& OutNewLockFlags, DeleteFunc Func)
	{
		if ((InCurrentLockFlags & LockFlags::WriteLock) == 0)
		{
			InCurrentLockFlags = ApplyLock(InCurrentLockFlags, LockFlags::WriteLock);
		}

		for (auto& KV : *BackfillMap)
		{
			Func(KV.Value);
		}

		int32 Discarded = BackfillMap->Num();

		// free anything still in the backfill map
		BackfillMap->Empty();

		if (CurrentMap == &Map1)
		{
			CurrentMap = &Map2;
			BackfillMap = &Map1;
		}
		else
		{
			CurrentMap = &Map1;
			BackfillMap = &Map2;
		}

		OutNewLockFlags = InCurrentLockFlags;
		return Discarded;
	}

public:
	uint32 ApplyLock(uint32 CurrentFlags, uint32 NewFlags)
	{
		bool IsLockedForRead = (CurrentFlags & LockFlags::ReadLock) != 0;
		bool IsLockedForWrite = (CurrentFlags & LockFlags::WriteLock) != 0;

		bool WantLockForRead = (NewFlags & LockFlags::ReadLock) != 0;
		bool WantLockForWrite = (NewFlags & LockFlags::WriteLock) != 0;
		

		// if already locked for write, nothing to do
		if (IsLockedForWrite && (WantLockForWrite || WantLockForRead))
		{
			return LockFlags::WriteLock;
		}

		// if locked for reads and that's all that's needed, d
		if (IsLockedForRead && WantLockForRead && !WantLockForWrite)
		{
			return LockFlags::ReadLock;
		}

		Unlock(CurrentFlags);
		
		// chance they asked for both Read/Write, so check this first
		if (WantLockForWrite)
		{
			LockObject.WriteLock();
		}
		else if (WantLockForRead)
		{
			LockObject.ReadLock();
		}	

		return NewFlags;
	}

	void Unlock(uint32 Flags)
	{
		bool LockedForRead = (Flags & LockFlags::ReadLock) != 0;
		bool LockedForWrite = (Flags & LockFlags::WriteLock) != 0;

		if (LockedForWrite)
		{
			LockObject.WriteUnlock();
		}
		else if (LockedForRead)
		{
			LockObject.ReadUnlock();
		}
	}

protected:

	/**
	*	Checks for the entry in our current map, and if not found the backfill. If the entry is in the backfill it is moved
	*	to the current map so it will not be discarded when DiscardUnusedEntries is called
	*/
	[[nodiscard]] bool  InternalFindWhileLocked(const KeyType& Key, ValueType& OutType, uint32 InCurrentLockFlags, uint32& OutFlags)
	{
		// by default we'll do exactly what was asked...

		bool LockedForWrite = (InCurrentLockFlags & LockFlags::WriteLock) != 0;
		bool LeaveWriteLockOnFailure = (InCurrentLockFlags & LockFlags::WriteLockOnAddFail) != 0;

		uint32 CurrentFlags = InCurrentLockFlags;

		checkf((CurrentFlags & LockFlags::ReadLock) != 0 
			|| (CurrentFlags & LockFlags::WriteLock) != 0, 
			TEXT("Cache is not locked for read or write during Find!"));

		// Do we have this?
		ValueType* Found = CurrentMap->Find(Key);

		// If not check the backfill,  if it's there remove it add it to our map. 
		if (!Found)
		{
			ValueType* BackfillFound = BackfillMap->Find(Key);

			// we either need to lock to adjust our cache, or lock because the user wants to...
			bool NeedWriteLock = (BackfillFound || LeaveWriteLockOnFailure);

			if (NeedWriteLock)
			{
				// lock the buffer (nop if we were already locked!)
				CurrentFlags = ApplyLock(CurrentFlags, CurrentFlags | LockFlags::WriteLock);

				// check again, there's a chance these may have changed filled between the unlock/lock
				// above..
				Found = CurrentMap->Find(Key);
				if (!Found)
				{
					BackfillFound = BackfillMap->Find(Key);
				}
			}

			// If we found a backfill, move it to the primary
			if (!Found && BackfillFound)
			{
				// if shared refs, add/remove order is important
				Found = &CurrentMap->Add(Key, *BackfillFound);
				BackfillMap->Remove(Key);
			}
		}

		if (Found)
		{
			OutType = *Found;
		}

		OutFlags = CurrentFlags;
		return Found != nullptr;
	}


	FRWLock			LockObject;
	TypedMap*		CurrentMap;
	TypedMap*		BackfillMap;
	TypedMap		Map1;
	TypedMap		Map2;
};