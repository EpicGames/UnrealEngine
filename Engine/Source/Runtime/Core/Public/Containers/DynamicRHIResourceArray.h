// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UnrealTypeTraits.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Array.h"
#include "CoreGlobals.h"
#include "Containers/ResourceArray.h"
#include "Serialization/MemoryImage.h"

/** alignment for supported resource types */
enum EResourceAlignment
{
	VERTEXBUFFER_ALIGNMENT	=DEFAULT_ALIGNMENT,
	INDEXBUFFER_ALIGNMENT	=DEFAULT_ALIGNMENT
};

/**
 * A array which allocates memory which can be used for UMA rendering resources.
 * In the dynamically bound RHI, it isn't any different from the default array type,
 * since none of the dynamically bound RHI implementations have UMA.
 *
 * @param Alignment - memory alignment to use for the allocation
 */
template< typename ElementType, uint32 Alignment = DEFAULT_ALIGNMENT>
class TResourceArray
	:	public FResourceArrayInterface
	,	public TArray<ElementType, TMemoryImageAllocator<Alignment>>
{
	using ParentArrayType = TArray<ElementType, TMemoryImageAllocator<Alignment>>;
	DECLARE_INLINE_TYPE_LAYOUT_EXPLICIT_BASES(TResourceArray, Virtual, FResourceArrayInterface, ParentArrayType);
public:
	using Super = ParentArrayType;

	/** 
	* Constructor 
	*/
	[[nodiscard]] explicit TResourceArray(bool InNeedsCPUAccess=false)
		:	Super()
		,	bNeedsCPUAccess(InNeedsCPUAccess)
	{
	}

	[[nodiscard]] TResourceArray(TResourceArray&&) = default;
	[[nodiscard]] TResourceArray(const TResourceArray&) = default;
	TResourceArray& operator=(TResourceArray&&) = default;
	TResourceArray& operator=(const TResourceArray&) = default;

	virtual ~TResourceArray() = default;


	// FResourceArrayInterface

	/**
	* @return A pointer to the resource data.
	*/
	[[nodiscard]] virtual const void* GetResourceData() const override
	{ 
		return &(*this)[0]; 
	}

	/**
	* @return size of resource data allocation
	*/
	[[nodiscard]] virtual uint32 GetResourceDataSize() const override
	{
		if (this->Num() > UINT32_MAX / sizeof(ElementType))
		{
			UE_LOG(LogCore, Fatal, TEXT("Resource data size too large for uint32, will overflow. Calculate with larger data type or use fewer elements. sizeof(ElementType): %" SIZE_T_FMT), sizeof(ElementType));
			return 0;
		}
		return this->Num() * sizeof(ElementType);
	}

	/**
	* Called on non-UMA systems after the RHI has copied the resource data, and no longer needs the CPU's copy.
	* Only discard the resource memory on clients, and if the CPU doesn't need access to it.
	* Non-clients can't discard the data because they may need to serialize it.
	*/
	virtual void Discard() override
	{
		if(!bNeedsCPUAccess && FPlatformProperties::RequiresCookedData() && !IsRunningCommandlet())
		{
			this->Empty();
		}
	}

	/**
	* @return true if the resource array is static and shouldn't be modified
	*/
	[[nodiscard]] virtual bool IsStatic() const override
	{
		return false;
	}

	/**
	* @return true if the resource keeps a copy of its resource data after the RHI resource has been created
	*/
	[[nodiscard]] virtual bool GetAllowCPUAccess() const override
	{
		return bNeedsCPUAccess;
	}

	/** 
	* Sets whether the resource array will be accessed by CPU. 
	*/
	virtual void SetAllowCPUAccess(bool bInNeedsCPUAccess) override
	{
		bNeedsCPUAccess = bInNeedsCPUAccess;
	}

	// Assignment operators.
	TResourceArray& operator=(const Super& Other)
	{
		Super::operator=(Other);
		return *this;
	}

	/**
	* Serialize data as a single block. See TArray::BulkSerialize for more info.
	*
	* IMPORTANT:
	*   - This is Overridden from UnrealTemplate.h TArray::BulkSerialize  Please make certain changes are propogated accordingly
	*
	* @param Ar	FArchive to bulk serialize this TArray to/from
	*/
	void BulkSerialize(FArchive& Ar, bool bForcePerElementSerialization = false)
	{
		Super::BulkSerialize(Ar, bForcePerElementSerialization);
	}

private:
	/** 
	* true if this array needs to be accessed by the CPU.  
	* If no CPU access is needed then the resource is freed once its RHI resource has been created
	*/
	LAYOUT_FIELD(bool, bNeedsCPUAccess);
};

/**
* Serializer for this class
* @param Ar - archive to serialize to
* @param ResourceArray - resource array data to serialize
*/
template<typename ElementType, uint32 Alignment>
FArchive& operator<<(FArchive& Ar,TResourceArray<ElementType,Alignment>& ResourceArray)
{
	return Ar << *(typename TResourceArray<ElementType,Alignment>::Super*)&ResourceArray;
}	
