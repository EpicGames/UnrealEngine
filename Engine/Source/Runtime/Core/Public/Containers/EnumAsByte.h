// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/IsPODType.h"
#include "Templates/TypeHash.h"

/**
 * Template to store enumeration values as bytes in a type-safe way.
 * Blueprint enums should either be enum classes (preferred):
 *		enum class EMyEnum : uint8 { One, Two }, which doesn't require wrapping in this template
 * or a namespaced enum
 *		namespace EMyEnum
 *		{
 *			enum Type // <- literally Type, not a placeholder
 *			{ One, Two };
 *		}
 */
template <class InEnumType>
class TEnumAsByte
{
	static_assert(std::is_enum_v<InEnumType> && std::is_convertible_v<InEnumType, int>, "TEnumAsByte is not intended for use with enum classes - please derive your enum class from uint8 instead.");

public:
	using EnumType = InEnumType;

	[[nodiscard]] constexpr TEnumAsByte() = default;
	[[nodiscard]] constexpr TEnumAsByte(const TEnumAsByte&) = default;
	constexpr TEnumAsByte& operator=(const TEnumAsByte&) = default;

	/**
	 * Constructor, initialize to the enum value.
	 *
	 * @param InValue value to construct with.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr TEnumAsByte( EnumType InValue )
		: Value(static_cast<uint8>(InValue))
	{
	}

	/**
	 * Constructor, initialize to the int32 value.
	 *
	 * @param InValue value to construct with.
	 */
	[[nodiscard]] explicit UE_FORCEINLINE_HINT constexpr TEnumAsByte( int32 InValue )
		: Value(static_cast<uint8>(InValue))
	{
	}

	/**
	 * Constructor, initialize to the uint8 value.
	 *
	 * @param InValue value to construct with.
	 */
	[[nodiscard]] explicit UE_FORCEINLINE_HINT constexpr TEnumAsByte( uint8 InValue )
		: Value(InValue)
	{
	}

public:
	/**
	 * Compares two enumeration values for equality.
	 *
	 * @param InValue The value to compare with.
	 * @return true if the two values are equal, false otherwise.
	 */
	[[nodiscard]] constexpr bool operator==( EnumType InValue ) const
	{
		return static_cast<EnumType>(Value) == InValue;
	}

	/**
	 * Compares two enumeration values for equality.
	 *
	 * @param InValue The value to compare with.
	 * @return true if the two values are equal, false otherwise.
	 */
	[[nodiscard]] constexpr bool operator==(TEnumAsByte InValue) const
	{
		return Value == InValue.Value;
	}

	/** Implicit conversion to EnumType. */
	[[nodiscard]] constexpr operator EnumType() const
	{
		return (EnumType)Value;
	}

public:

	/**
	 * Gets the enumeration value.
	 *
	 * @return The enumeration value.
	 */
	[[nodiscard]] constexpr EnumType GetValue() const
	{
		return (EnumType)Value;
	}

	/**
	 * Gets the integer enumeration value.
	 *
	 * @return The enumeration value.
	 */
	[[nodiscard]] constexpr uint8 GetIntValue() const
	{
		return Value;
	}

private:

	/** Holds the value as a byte. **/
	uint8 Value;
};

template<class T>
[[nodiscard]] UE_FORCEINLINE_HINT constexpr uint32 GetTypeHash(const TEnumAsByte<T>& Enum)
{
	return GetTypeHash((uint8)Enum.GetValue());
}


template<class T> struct TIsPODType<TEnumAsByte<T>> { enum { Value = true }; };
