// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainerAllocationPolicies.h"
#include "CoreTypes.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PlatformCrt.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/MemoryImageWriter.h"
#include "Serialization/MemoryLayout.h"
#include "Templates/UnrealTemplate.h"

#include <initializer_list>

class FMemoryImageWriter;
class FMemoryUnfreezeContent;
class FPointerTableBase;
class FSHA1;

inline uint32 MurmurFinalize32(uint32 Hash)
{
	Hash ^= Hash >> 16;
	Hash *= 0x85ebca6b;
	Hash ^= Hash >> 13;
	Hash *= 0xc2b2ae35;
	Hash ^= Hash >> 16;
	return Hash;
}

inline uint64 MurmurFinalize64(uint64 Hash)
{
	Hash ^= Hash >> 33;
	Hash *= 0xff51afd7ed558ccdull;
	Hash ^= Hash >> 33;
	Hash *= 0xc4ceb9fe1a85ec53ull;
	Hash ^= Hash >> 33;
	return Hash;
}

inline uint32 Murmur32( std::initializer_list< uint32 > InitList )
{
	uint32 Hash = 0;
	for( auto Element : InitList )
	{
		Element *= 0xcc9e2d51;
		Element = ( Element << 15 ) | ( Element >> (32 - 15) );
		Element *= 0x1b873593;
    
		Hash ^= Element;
		Hash = ( Hash << 13 ) | ( Hash >> (32 - 13) );
		Hash = Hash * 5 + 0xe6546b64;
	}

	return MurmurFinalize32( Hash );
}

inline uint64 Murmur64( std::initializer_list< uint64 > InitList )
{
	uint64 Hash = 0;
	for( auto Element : InitList )
	{
		Element *= 0x87c37b91114253d5ull;
		Element = ( Element << 31 ) | ( Element >> (64 - 31) );
		Element *= 0x4cf5ad432745937full;

		Hash ^= Element;
		Hash = ( Hash << 27 ) | ( Hash >> (64 - 27) );
		Hash = Hash * 5 + 0x52dce729;
	}

	return MurmurFinalize64( Hash );
}

/*-----------------------------------------------------------------------------
	Statically sized hash table, used to index another data structure.
	Vastly simpler and faster than TMap.

	Example find:

	uint16 Key = HashFunction( ID );
	for( uint16 i = HashTable.First( Key ); HashTable.IsValid( i ); i = HashTable.Next( i ) )
	{
		if( Array[i].ID == ID )
		{
			return Array[i];
		}
	}
-----------------------------------------------------------------------------*/
template< uint16 HashSize, uint16 IndexSize >
class TStaticHashTable
{
public:
				TStaticHashTable();
				TStaticHashTable(ENoInit);

	
	void		Clear();

	// Functions used to search
	uint16		First( uint16 Key ) const;
	uint16		Next( uint16 Index ) const;
	bool		IsValid( uint16 Index ) const;
	
	void		Add( uint16 Key, uint16 Index );
	void		Remove( uint16 Key, uint16 Index );

protected:
	uint16		Hash[ HashSize ];
	uint16		NextIndex[ IndexSize ];
};

template< uint16 HashSize, uint16 IndexSize >
inline TStaticHashTable< HashSize, IndexSize >::TStaticHashTable()
{
	static_assert( ( HashSize & (HashSize - 1) ) == 0, "Hash size must be power of 2" );
	static_assert( IndexSize - 1 < 0xffff, "Index 0xffff is reserved" );
	Clear();
}

template< uint16 HashSize, uint16 IndexSize >
inline TStaticHashTable< HashSize, IndexSize >::TStaticHashTable(ENoInit)
{
	static_assert((HashSize & (HashSize - 1)) == 0, "Hash size must be power of 2");
	static_assert(IndexSize - 1 < 0xffff, "Index 0xffff is reserved");
}

template< uint16 HashSize, uint16 IndexSize >
UE_FORCEINLINE_HINT void TStaticHashTable< HashSize, IndexSize >::Clear()
{
	FMemory::Memset( Hash, 0xff, HashSize * 2 );
}

// First in hash chain
template< uint16 HashSize, uint16 IndexSize >
inline uint16 TStaticHashTable< HashSize, IndexSize >::First( uint16 Key ) const
{
	Key &= HashSize - 1;
	return Hash[ Key ];
}

// Next in hash chain
template< uint16 HashSize, uint16 IndexSize >
inline uint16 TStaticHashTable< HashSize, IndexSize >::Next( uint16 Index ) const
{
	checkSlow( Index < IndexSize );
	return NextIndex[ Index ];
}

template< uint16 HashSize, uint16 IndexSize >
UE_FORCEINLINE_HINT bool TStaticHashTable< HashSize, IndexSize >::IsValid( uint16 Index ) const
{
	return Index != 0xffff;
}

template< uint16 HashSize, uint16 IndexSize >
inline void TStaticHashTable< HashSize, IndexSize >::Add( uint16 Key, uint16 Index )
{
	checkSlow( Index < IndexSize );

	Key &= HashSize - 1;
	NextIndex[ Index ] = Hash[ Key ];
	Hash[ Key ] = Index;
}

template< uint16 HashSize, uint16 IndexSize >
inline void TStaticHashTable< HashSize, IndexSize >::Remove( uint16 Key, uint16 Index )
{
	checkSlow( Index < IndexSize );

	Key &= HashSize - 1;

	if( Hash[Key] == Index )
	{
		// Head of chain
		Hash[Key] = NextIndex[ Index ];
	}
	else
	{
		for( uint16 i = Hash[Key]; IsValid(i); i = NextIndex[i] )
		{
			if( NextIndex[i] == Index )
			{
				// Next = Next->Next
				NextIndex[i] = NextIndex[ Index ];
				break;
			}
		}
	}
}

/*-----------------------------------------------------------------------------
	Dynamically sized hash table, used to index another data structure.
	Vastly simpler and faster than TMap.

	Example find:

	uint32 Key = HashFunction( ID );
	for( uint32 i = HashTable.First( Key ); HashTable.IsValid( i ); i = HashTable.Next( i ) )
	{
		if( Array[i].ID == ID )
		{
			return Array[i];
		}
	}
-----------------------------------------------------------------------------*/
class FHashTable
{
public:
					FHashTable( uint32 InHashSize = 1024, uint32 InIndexSize = 0 );
					FHashTable(const FHashTable& Other);
					FHashTable(FHashTable&& Other);
					~FHashTable();

	void			Clear();
	void			Clear( uint32 InHashSize, uint32 InIndexSize = 0 );
	void			Free();
	/** 
	 * Increases or decreases the size of the index but not the hash lookup.
	 * If the previous size was empty, allocates the hash at its desired size.
	 */
	CORE_API void	Resize( uint32 NewIndexSize );
	inline uint32	GetIndexSize() const { return IndexSize; }
	inline uint32	GetHashSize() const { return HashSize; }
	CORE_API SIZE_T	GetAllocatedSize() const;

	// Functions used to search
	uint32			First( uint32 Key ) const;
	uint32			Next( uint32 Index ) const;
	bool			IsValid( uint32 Index ) const;
	
	void			Add( uint32 Key, uint32 Index );
	// Safe to call concurrently with other threads calling Add_Concurrent with different values for Index.
	void			Add_Concurrent( uint32 Key, uint32 Index );
	void			Remove( uint32 Key, uint32 Index );

	// Average # of compares per search
	CORE_API float	AverageSearch() const;

	FHashTable&		operator=(const FHashTable& Other);
	FHashTable&		operator=(FHashTable&& Other);

protected:
	// Avoids allocating hash until first add
	CORE_API static uint32	EmptyHash[1];

	uint32			HashSize;
	uint32			HashMask;
	uint32			IndexSize;

	uint32*			Hash;
	uint32*			NextIndex;
};


inline FHashTable::FHashTable( uint32 InHashSize, uint32 InIndexSize )
	: HashSize( InHashSize )
	, HashMask( 0 )
	, IndexSize( InIndexSize )
	, Hash( EmptyHash )
	, NextIndex( nullptr )
{
	check( HashSize > 0 );
	check( FMath::IsPowerOfTwo( HashSize ) );
	
	if( IndexSize )
	{
		HashMask = HashSize - 1;
		
		Hash = new uint32[ HashSize ];
		NextIndex = new uint32[ IndexSize ];

		FMemory::Memset( Hash, 0xff, HashSize * 4 );
	}
}

inline FHashTable::FHashTable( const FHashTable& Other )
	: HashSize( Other.HashSize )
	, HashMask( Other.HashMask )
	, IndexSize( Other.IndexSize )
	, Hash( EmptyHash )
{
	if( IndexSize )
	{
		Hash = new uint32[ HashSize ];
		NextIndex = new uint32[ IndexSize ];

		FMemory::Memcpy( Hash, Other.Hash, HashSize * 4 );
		FMemory::Memcpy( NextIndex, Other.NextIndex, IndexSize * 4 );
	}
}

inline FHashTable::FHashTable(FHashTable&& Other )
	: HashSize( Other.HashSize )
	, HashMask( Other.HashMask )
	, IndexSize( Other.IndexSize )
	, Hash(Other.Hash)
	, NextIndex(Other.NextIndex)
{
	Other.HashSize = 0;
	Other.HashMask = 0;
	Other.IndexSize = 0;
	Other.Hash = EmptyHash;
	Other.NextIndex = nullptr;
}

inline FHashTable& FHashTable::operator=(const FHashTable& Other)
{
	Free();
	HashSize = Other.HashSize;
	HashMask = Other.HashMask;
	IndexSize = Other.IndexSize;

	if (IndexSize)
	{
		Hash = new uint32[HashSize];
		NextIndex = new uint32[IndexSize];

		FMemory::Memcpy(Hash, Other.Hash, HashSize * 4);
		FMemory::Memcpy(NextIndex, Other.NextIndex, IndexSize * 4);
	}
	return *this;
}

inline FHashTable& FHashTable::operator=(FHashTable&& Other)
{
	Free();

	HashSize = Other.HashSize;
	HashMask = Other.HashMask;
	IndexSize = Other.IndexSize;
	Hash = Other.Hash;
	NextIndex = Other.NextIndex;

	Other.HashSize = 0;
	Other.HashMask = 0;
	Other.IndexSize = 0;
	Other.Hash = EmptyHash;
	Other.NextIndex = nullptr;
	return *this;
}

UE_FORCEINLINE_HINT FHashTable::~FHashTable()
{
	Free();
}

inline void FHashTable::Clear()
{
	if( IndexSize )
	{
		FMemory::Memset( Hash, 0xff, HashSize * 4 );
	}
}

inline void FHashTable::Clear( uint32 InHashSize, uint32 InIndexSize )
{
	Free();

	HashSize = InHashSize;
	IndexSize = InIndexSize;

	check( HashSize > 0 );
	check( FMath::IsPowerOfTwo( HashSize ) );

	if( IndexSize )
	{
		HashMask = HashSize - 1;
		
		Hash = new uint32[ HashSize ];
		NextIndex = new uint32[ IndexSize ];

		FMemory::Memset( Hash, 0xff, HashSize * 4 );
	}
}

inline void FHashTable::Free()
{
	if( IndexSize )
	{
		HashMask = 0;
		IndexSize = 0;
		
		delete[] Hash;
		Hash = EmptyHash;
		
		delete[] NextIndex;
		NextIndex = nullptr;
	}
} 

// First in hash chain
inline uint32 FHashTable::First( uint32 Key ) const
{
	Key &= HashMask;
	return Hash[ Key ];
}

// Next in hash chain
inline uint32 FHashTable::Next( uint32 Index ) const
{
	checkSlow( Index < IndexSize );
	checkSlow( NextIndex[Index] != Index ); // check for corrupt tables
	return NextIndex[ Index ];
}

UE_FORCEINLINE_HINT bool FHashTable::IsValid( uint32 Index ) const
{
	return Index != ~0u;
}

inline void FHashTable::Add( uint32 Key, uint32 Index )
{
	if( Index >= IndexSize )
	{
		Resize( FMath::Max< uint32 >( 32u, FMath::RoundUpToPowerOfTwo( Index + 1 ) ) );
	}

	Key &= HashMask;
	NextIndex[ Index ] = Hash[ Key ];
	Hash[ Key ] = Index;
}

// Safe for many threads to add concurrently.
// Not safe to search the table while other threads are adding.
// Will not resize. Only use for presized tables.
inline void FHashTable::Add_Concurrent( uint32 Key, uint32 Index )
{
	check( Index < IndexSize );

	Key &= HashMask;
	NextIndex[ Index ] = FPlatformAtomics::InterlockedExchange( (int32*)&Hash[ Key ], Index );
}

inline void FHashTable::Remove( uint32 Key, uint32 Index )
{
	if( Index >= IndexSize )
	{
		return;
	}

	Key &= HashMask;

	if( Hash[Key] == Index )
	{
		// Head of chain
		Hash[Key] = NextIndex[ Index ];
	}
	else
	{
		for( uint32 i = Hash[Key]; IsValid(i); i = NextIndex[i] )
		{
			if( NextIndex[i] == Index )
			{
				// Next = Next->Next
				NextIndex[i] = NextIndex[ Index ];
				break;
			}
		}
	}
}

template<typename InAllocator>
class THashTable
{
public:
	using Allocator = InAllocator;

	using ElementAllocatorType = std::conditional_t<
		Allocator::NeedsElementType,
		typename Allocator::template ForElementType<uint32>,
		typename Allocator::ForAnyElementType
	>;

	explicit THashTable(uint32 InHashSize = 1024, uint32 InIndexSize = 0);
	THashTable(const THashTable& Other) = delete;
	THashTable(THashTable&& Other) { MoveAssign(MoveTemp(Other)); }
	~THashTable();

	THashTable& operator=(const THashTable& Other) = delete;
	THashTable& operator=(THashTable&& Other) { return MoveAssign(MoveTemp(Other)); }

	THashTable&		MoveAssign(THashTable&& Other);
	void			Clear();
	void			Resize(uint32 NewIndexSize);

	const uint32*	GetNextIndices() const { return (uint32*)NextIndex.GetAllocation(); }

	// Functions used to search
	uint32			First(uint16 Key) const;
	uint32			Next(uint32 Index) const;
	bool			IsValid(uint32 Index) const;

	void			Add(uint16 Key, uint32 Index);
	void			Remove(uint16 Key, uint32 Index);

private:
	UE_FORCEINLINE_HINT uint32 HashAt(uint32 Index) const { return ((uint32*)Hash.GetAllocation())[Index]; }
	UE_FORCEINLINE_HINT uint32 NextIndexAt(uint32 Index) const { return ((uint32*)NextIndex.GetAllocation())[Index]; }
	UE_FORCEINLINE_HINT uint32& HashAt(uint32 Index) { return ((uint32*)Hash.GetAllocation())[Index]; }
	UE_FORCEINLINE_HINT uint32& NextIndexAt(uint32 Index) { return ((uint32*)NextIndex.GetAllocation())[Index]; }

	ElementAllocatorType	Hash;
	ElementAllocatorType	NextIndex;
	uint32					HashMask;
	uint32					IndexSize;

public:
	void WriteMemoryImage(FMemoryImageWriter& Writer) const
	{
		if constexpr (TAllocatorTraits<Allocator>::SupportsFreezeMemoryImage)
		{
			this->Hash.WriteMemoryImage(Writer, StaticGetTypeLayoutDesc<uint32>(), this->HashMask + 1u);
			this->NextIndex.WriteMemoryImage(Writer, StaticGetTypeLayoutDesc<uint32>(), this->IndexSize);
			Writer.WriteBytes(this->HashMask);
			Writer.WriteBytes(this->IndexSize);
		}
		else
		{
			check(false);
		}
	}

	void CopyUnfrozen(const FMemoryUnfreezeContent& Context, void* Dst) const
	{
		if constexpr (TAllocatorTraits<Allocator>::SupportsFreezeMemoryImage)
		{
			THashTable* DstTable = ::new(Dst) THashTable(this->HashMask + 1u, this->IndexSize);
			FMemory::Memcpy(DstTable->Hash.GetAllocation(), this->Hash.GetAllocation(), (this->HashMask + 1u) * 4);
			FMemory::Memcpy(DstTable->NextIndex.GetAllocation(), this->NextIndex.GetAllocation(), this->IndexSize * 4);
		}
		else
		{
			::new(Dst) THashTable();
		}
	}
};

template<typename InAllocator>
inline THashTable<InAllocator>::THashTable(uint32 InHashSize, uint32 InIndexSize)
	: HashMask(InHashSize - 1u)
	, IndexSize(InIndexSize)
{
	checkf(InHashSize > 0u && InHashSize <= 0x10000, TEXT("InHashSize out of range: %u, InIndexSize %u"), InHashSize, InIndexSize);
	checkf(FMath::IsPowerOfTwo(InHashSize), TEXT("InHashSize (%u) is not a power of two, InIndexSize %u"), InHashSize, InIndexSize);

	Hash.ResizeAllocation(0, InHashSize, sizeof(uint32));
	FMemory::Memset(Hash.GetAllocation(), 0xff, InHashSize * 4);

	if (IndexSize)
	{
		NextIndex.ResizeAllocation(0, IndexSize, sizeof(uint32));
	}
}

template<typename InAllocator>
UE_FORCEINLINE_HINT THashTable<InAllocator>::~THashTable()
{
}

template<typename InAllocator>
THashTable<InAllocator>& THashTable<InAllocator>::MoveAssign(THashTable&& Other)
{
	Hash.MoveToEmpty(Other.Hash);
	NextIndex.MoveToEmpty(Other.NextIndex);
	HashMask = Other.HashMask;
	IndexSize = Other.IndexSize;
	Other.HashMask = 0u;
	Other.IndexSize = 0u;
	return *this;
}

template<typename InAllocator>
inline void THashTable<InAllocator>::Clear()
{
	if (IndexSize)
	{
		FMemory::Memset(Hash.GetAllocation(), 0xff, (HashMask + 1u) * 4);
	}
}

// First in hash chain
template<typename InAllocator>
inline uint32 THashTable<InAllocator>::First(uint16 Key) const
{
	Key &= HashMask;
	return HashAt(Key);
}

// Next in hash chain
template<typename InAllocator>
inline uint32 THashTable<InAllocator>::Next(uint32 Index) const
{
	checkSlow(Index < IndexSize);
	const uint32 Next = NextIndexAt(Index);
	checkSlow(Next != Index); // check for corrupt tables
	return Next;
}

template<typename InAllocator>
UE_FORCEINLINE_HINT bool THashTable<InAllocator>::IsValid(uint32 Index) const
{
	return Index != ~0u;
}

template<typename InAllocator>
inline void THashTable<InAllocator>::Add(uint16 Key, uint32 Index)
{
	if (Index >= IndexSize)
	{
		Resize(FMath::Max<uint32>(32u, FMath::RoundUpToPowerOfTwo(Index + 1)));
	}

	Key &= HashMask;
	NextIndexAt(Index) = HashAt(Key);
	HashAt(Key) = Index;
}

template<typename InAllocator>
inline void THashTable<InAllocator>::Remove(uint16 Key, uint32 Index)
{
	if (Index >= IndexSize)
	{
		return;
	}

	Key &= HashMask;
	if (HashAt(Key) == Index)
	{
		// Head of chain
		HashAt(Key) = NextIndexAt(Index);
	}
	else
	{
		for (uint32 i = HashAt(Key); IsValid(i); i = NextIndexAt(i))
		{
			if (NextIndexAt(i) == Index)
			{
				// Next = Next->Next
				NextIndexAt(i) = NextIndexAt(Index);
				break;
			}
		}
	}
}

template<typename InAllocator>
void THashTable<InAllocator>::Resize(uint32 NewIndexSize)
{
	if (NewIndexSize != IndexSize)
	{
		NextIndex.ResizeAllocation(IndexSize, NewIndexSize, sizeof(uint32));
		IndexSize = NewIndexSize;
	}
}

namespace Freeze
{
	template<typename InAllocator>
	void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const THashTable<InAllocator>& Object, const FTypeLayoutDesc&)
	{
		Object.WriteMemoryImage(Writer);
	}

	template<typename InAllocator>
	uint32 IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const THashTable<InAllocator>& Object, void* OutDst)
	{
		Object.CopyUnfrozen(Context, OutDst);
		return sizeof(Object);
	}

	template<typename InAllocator>
	uint32 IntrinsicAppendHash(const THashTable<InAllocator>* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		return AppendHashForNameAndSize(TypeDesc.Name, sizeof(THashTable<InAllocator>), Hasher);
	}

	template<typename InAllocator>
	uint32 IntrinsicGetTargetAlignment(const THashTable<InAllocator>* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams)
	{
		// Assume alignment of hash-table is drive by pointer
		return FMath::Min(8u, LayoutParams.MaxFieldAlignment);
	}
}

DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT(template <typename InAllocator>, THashTable<InAllocator>);
