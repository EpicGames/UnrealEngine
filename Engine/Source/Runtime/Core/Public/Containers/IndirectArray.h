// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTypeTraits.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Array.h"

/*-----------------------------------------------------------------------------
	Indirect array.
	Same as a TArray, but stores pointers to the elements, to allow
	resizing the array index without relocating the actual elements.
-----------------------------------------------------------------------------*/

template<typename T,typename Allocator = FDefaultAllocator>
class TIndirectArray
{
public:
	typedef T                        ElementType;
	typedef TArray<void*, Allocator> InternalArrayType;
	typedef Allocator 				 AllocatorType;

	/** Default constructors. */
	[[nodiscard]] TIndirectArray() = default;
	[[nodiscard]] TIndirectArray(TIndirectArray&&) = default;

	/**
	 * Copy constructor.
	 *
	 * @param Other Other array to copy from.
	 */
	[[nodiscard]] TIndirectArray(const TIndirectArray& Other)
	{
		for (auto& Item : Other)
		{
			Add(new T(Item));
		}
	}

	/**
	 * Assignment operators.
	 *
	 * @param Other Other array to assign with.
	 */
	TIndirectArray& operator=(const TIndirectArray& Other)
	{
		if (&Other != this)
		{
			Empty(Other.Num());
			for (auto& Item : Other)
			{
				Add(new T(Item));
			}
		}

		return *this;
	}
	TIndirectArray &operator=(TIndirectArray && Other)
	{
		if (&Other != this)
		{
			Empty();
			Array = MoveTemp(Other.Array);
		}

		return *this;
	}


	/** Destructor. */
	~TIndirectArray()
	{
		Empty();
	}

	/**
	 * Returns true if the array is empty and contains no elements.
	 *
	 * @returns True if the array is empty.
	 * @see Num
	 */
	[[nodiscard]] bool IsEmpty() const
	{
		return Array.IsEmpty();
	}

	/**
	 * Gets number of elements in array.
	 *
	 * @returns Number of elements in array.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Num() const
	{
		return Array.Num();
	}

	/**
	 * Helper function for returning a typed pointer to the first array entry.
	 *
	 * @returns Pointer to first array entry or nullptr if this->ArrayMax == 0.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T** GetData()
	{
		return (T**)Array.GetData();
	}

	/**
	 * Helper function for returning a typed pointer to the first array entry.
	 *
	 * @returns Pointer to first array entry or nullptr if this->ArrayMax == 0.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const T** GetData() const
	{
		return (const T**)Array.GetData();
	}

	/**
	 * Helper function returning the size of the inner type.
	 *
	 * @returns Size in bytes of array type.
	 */
	[[nodiscard]] static constexpr uint32 GetTypeSize()
	{
		return sizeof(T*);
	}

	/**
	 * Bracket array access operator.
	 *
	 * @param Index Position of element to return.
	 * @returns Reference to element in array at given position.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T& operator[](int32 Index)
	{
		return *(T*)Array[Index];
	}

	/**
	 * Bracket array access operator.
	 *
	 * Const version.
	 *
	 * @param Index Position of element to return.
	 * @returns Reference to element in array at given position.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const T& operator[](int32 Index) const
	{
		return *(T*)Array[Index];
	}

	/**
	 * Returns n-th last element from the array.
	 *
	 * @param IndexFromTheEnd (Optional) Index from the end of array (default = 0).
	 * @returns Reference to n-th last element from the array.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& Last(int32 IndexFromTheEnd = 0)
	{
		return *(T*)Array.Last(IndexFromTheEnd);
	}

	/**
	 * Returns n-th last element from the array.
	 *
	 * Const version.
	 *
	 * @param IndexFromTheEnd (Optional) Index from the end of array (default = 0).
	 * @returns Reference to n-th last element from the array.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType& Last(int32 IndexFromTheEnd = 0) const
	{
		return *(T*)Array.Last(IndexFromTheEnd);
	}

	/**
	 * Shrinks the array's used memory to smallest possible to store elements
	 * currently in it.
	 */
	void Shrink()
	{
		Array.Shrink();
	}

	/**
	 * Resets the array to the new given size. It calls the destructors on held
	 * items.
	 *
	 * @param NewSize (Optional) The expected usage size after calling this
	 *		function. Default is 0.
	 */
	void Reset(int32 NewSize = 0)
	{
		DestructAndFreeItems();
		Array.Reset(NewSize);
	}

	/**
	 * Special serialize function passing the owning UObject along as required
	 * by FUnytpedBulkData serialization.
	 *
	 * @param Ar Archive to serialize with.
	 * @param Owner UObject this structure is serialized within.
	 */
	void Serialize(FArchive& Ar, UObject* Owner)
	{
		CountBytes(Ar);
		if (Ar.IsLoading())
		{
			// Load array.
			int32 NewNum = 0;
			Ar << NewNum;
			Empty(NewNum);
			for (int32 Index = 0; Index < NewNum; Index++)
			{
				Add(new T);
			}
			for (int32 Index = 0; Index < NewNum; Index++)
			{
				(*this)[Index].Serialize(Ar, Owner, Index);
			}
		}
		else
		{
			// Save array.
			int32 Num = Array.Num();
			Ar << Num;
			for (int32 Index = 0; Index < Num; Index++)
			{
				(*this)[Index].Serialize(Ar, Owner, Index);
			}
		}
	}

	/**
	 * Count bytes needed to serialize this array.
	 *
	 * @param Ar Archive to count for.
	 */
	void CountBytes(FArchive& Ar) const
	{
		Array.CountBytes(Ar);
	}

	/**
	 * Removes an element at given location optionally shrinking
	 * the array.
	 *
	 * @param Index Location in array of the element to remove.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 */
	void RemoveAt(int32 Index, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<Allocator>())
	{
		check(Index >= 0);
		check(Index < Array.Num());
		T** Element = GetData() + Index;
		delete *Element;
		Array.RemoveAt(Index, AllowShrinking);
	}

	/**
	 * Removes elements at given location optionally shrinking
	 * the array.
	 *
	 * @param Index Location in array of the first element to remove.
	 * @param Count Number of elements to remove.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 */
	void RemoveAt(int32 Index, int32 Count, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<Allocator>())
	{
		check(Index >= 0);
		check(Index <= Array.Num());
		check(Index + Count <= Array.Num());
		T** Element = GetData() + Index;
		for (int32 ElementId = Count; ElementId; --ElementId)
		{
			delete *Element;
			++Element;
		}
		Array.RemoveAt(Index, Count, AllowShrinking);
	}

	/**
	 * Removes an element at given location optionally shrinking
	 * the array.
	 *
	 * This version is much more efficient than RemoveAt (O(Count) instead of
	 * O(ArrayNum)), but does not preserve the order.
	 *
	 * @param Index Location in array of the element to remove.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 */
	void RemoveAtSwap(int32 Index, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<Allocator>())
	{
		check(Index >= 0);
		check(Index < Array.Num());
		T** Element = GetData() + Index;
		delete *Element;
		Array.RemoveAtSwap(Index, AllowShrinking);
	}

	/**
	 * Removes elements at given location optionally shrinking
	 * the array.
	 *
	 * This version is much more efficient than RemoveAt (O(Count) instead of
	 * O(ArrayNum)), but does not preserve the order.
	 *
	 * @param Index Location in array of the rirst element to remove.
	 * @param Count Number of elements to remove.
	 * @param AllowShrinking (Optional) By default, arrays with large amounts of slack will automatically shrink.
	 *                       Use FNonshrinkingAllocator or pass EAllowShrinking::No to prevent this.
	 */
	void RemoveAtSwap(int32 Index, int32 Count, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<Allocator>())
	{
		check(Index >= 0);
		check(Index <= Array.Num());
		check(Index + Count <= Array.Num());
		T** Element = GetData() + Index;
		for (int32 ElementId = Count; ElementId; --ElementId)
		{
			delete *Element;
			++Element;
		}
		Array.RemoveAtSwap(Index, Count, AllowShrinking);
	}

	/**
	 * Element-wise array element swap.
	 *
	 * This version is doing more sanity checks than SwapMemory.
	 *
	 * @param FirstIndexToSwap Position of the first element to swap.
	 * @param SecondIndexToSwap Position of the second element to swap.
	 */
	void Swap(int32 FirstIndexToSwap, int32 SecondIndexToSwap)
	{
		Array.Swap(FirstIndexToSwap, SecondIndexToSwap);
	}

	/**
	 * Empties the array. It calls the destructors on held items.
	 *
	 * @param Slack (Optional) The expected usage size after empty operation.
	 *              Default is 0.
	 */
	void Empty(int32 Slack = 0)
	{
		DestructAndFreeItems();
		Array.Empty(Slack);
	}

	/**
	 * Adds a new item to the end of the array, possibly reallocating the
	 * whole array to fit.
	 *
	 * @param Item The item to add.
	 * @returns Index to the new item.
	 */
	UE_FORCEINLINE_HINT int32 Add(T* Item)
	{
		return Array.Add(Item);
	}

	/**
	 * Inserts a given element into the array at given location.
	 *
	 * @param Item The element to insert.
	 * @param Index Tells where to insert the new elements.
	 */
	UE_FORCEINLINE_HINT void Insert(T* Item, int32 Index)
	{
		Array.Insert(Item, Index);
	}

	/**
	 * Reserves memory such that the array can contain at least Number elements.
	 *
	 * @param Number The number of elements that the array should be able to
	 *               contain after allocation.
	 */
	UE_FORCEINLINE_HINT void Reserve(int32 Number)
	{
		Array.Reserve(Number);
	}

	/**
	 * Tests if index is valid, i.e. greater than zero and less than number of
	 * elements in array.
	 *
	 * @param Index Index to test.
	 * @returns True if index is valid. False otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValidIndex(int32 Index) const
	{
		return Array.IsValidIndex(Index);
	}

	/**
	 * Helper function to return the amount of memory allocated by this container
	 *
	 * @returns Number of bytes allocated by this container.
	 */
	[[nodiscard]] SIZE_T GetAllocatedSize() const
	{
		return Array.Max() * sizeof(T*) + Array.Num() * sizeof(T);
	}

	// Iterators
	typedef TIndexedContainerIterator<      TIndirectArray,       ElementType, int32> TIterator;
	typedef TIndexedContainerIterator<const TIndirectArray, const ElementType, int32> TConstIterator;

	/**
	 * Creates an iterator for the contents of this array.
	 *
	 * @returns The iterator.
	 */
	[[nodiscard]] TIterator CreateIterator()
	{
		return TIterator(*this);
	}

	/**
	 * Creates a const iterator for the contents of this array.
	 *
	 * @returns The const iterator.
	 */
	[[nodiscard]] TConstIterator CreateConstIterator() const
	{
		return TConstIterator(*this);
	}

private:

	/**
	 * Calls destructor and frees memory on every element in the array.
	 */
	void DestructAndFreeItems()
	{
		T** Element = GetData();
		for (int32 Index = Array.Num(); Index; --Index)
		{
			delete *Element;
			++Element;
		}
	}

public:
	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TDereferencingIterator<      ElementType, typename InternalArrayType::RangedForIteratorType     > begin()       { return TDereferencingIterator<      ElementType, typename InternalArrayType::RangedForIteratorType     >(Array.begin()); }
	[[nodiscard]] UE_FORCEINLINE_HINT TDereferencingIterator<const ElementType, typename InternalArrayType::RangedForConstIteratorType> begin() const { return TDereferencingIterator<const ElementType, typename InternalArrayType::RangedForConstIteratorType>(Array.begin()); }
	[[nodiscard]] UE_FORCEINLINE_HINT TDereferencingIterator<      ElementType, typename InternalArrayType::RangedForIteratorType     > end  ()       { return TDereferencingIterator<      ElementType, typename InternalArrayType::RangedForIteratorType     >(Array.end()); }
	[[nodiscard]] UE_FORCEINLINE_HINT TDereferencingIterator<const ElementType, typename InternalArrayType::RangedForConstIteratorType> end  () const { return TDereferencingIterator<const ElementType, typename InternalArrayType::RangedForConstIteratorType>(Array.end()); }

private:

	InternalArrayType Array;
};


/**
* Serialization operator for TIndirectArray.
*
* @param Ar Archive to serialize with.
* @param A Array to serialize.
* @returns Passing down serializing archive.
*/
template<typename T,typename Allocator>
FArchive& operator<<(FArchive& Ar, TIndirectArray<T, Allocator>& A)
{
	A.CountBytes(Ar);
	if (Ar.IsLoading())
	{
		// Load array.
		int32 NewNum;
		Ar << NewNum;
		A.Empty(NewNum);
		for (int32 Index = 0; Index < NewNum; Index++)
		{
			T* NewElement = new T;
			Ar << *NewElement;
			A.Add(NewElement);
		}
	}
	else
	{
		// Save array.
		int32 Num = A.Num();
		Ar << Num;
		for (int32 Index = 0; Index < Num; Index++)
		{
			Ar << A[Index];
		}
	}
	return Ar;
}
