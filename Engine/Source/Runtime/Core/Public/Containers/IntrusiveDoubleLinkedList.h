// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"


// Forward declarations
template <class NodeType>
class TIntrusiveDoubleLinkedListIterator;

template <class ElementType, class ContainerType>
class TIntrusiveDoubleLinkedList;


/**
 * Node of an intrusive double linked list
 * Structs/classes must inherit this, to use it, e.g: struct FMyStruct : public TIntrusiveDoubleLinkedListNode<FMyStruct>
 * TIntrusiveDoubleLinkedListNode can be inherited multiple times, ex: if ElementType needs to be stored in several lists at once
 * by specifying a different ContainerType template parameter to distinguish the nodes.
 */
template <class InElementType, class ContainerType = InElementType>
class TIntrusiveDoubleLinkedListNode
{
public:
	using NodeType = TIntrusiveDoubleLinkedListNode<InElementType, ContainerType>;
	using ElementType = InElementType;

	[[nodiscard]] TIntrusiveDoubleLinkedListNode()
		: Next(GetThisElement())
		, Prev(GetThisElement())
	{}

	UE_FORCEINLINE_HINT void Reset()
	{
		Next = Prev = GetThisElement();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool IsInList() const
	{
		return Next != GetThisElement();
	}
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType* GetNext() const
	{
		return Next;
	}
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType* GetPrev() const
	{
		return Prev;
	}

	/**
	 * Removes this element from the list in constant time.
	 */
	inline void Remove()
	{
		static_cast<NodeType*>(Next)->Prev = Prev;
		static_cast<NodeType*>(Prev)->Next = Next;
		Next = Prev = GetThisElement();
	}

	/**
	 * Insert this node after the specified node
	 */
	inline void InsertAfter(ElementType* NewPrev)
	{
		ElementType* NewNext = static_cast<NodeType*>(NewPrev)->Next;
		Next = NewNext;
		Prev = NewPrev;
		static_cast<NodeType*>(NewNext)->Prev = GetThisElement();
		static_cast<NodeType*>(NewPrev)->Next = GetThisElement();
	}

	/**
	 * Insert this node before the specified node
	 */
	inline void InsertBefore(ElementType* NewNext)
	{
		ElementType* NewPrev = static_cast<NodeType*>(NewNext)->Prev;
		Next = NewNext;
		Prev = NewPrev;
		static_cast<NodeType*>(NewNext)->Prev = GetThisElement();
		static_cast<NodeType*>(NewPrev)->Next = GetThisElement();
	}

protected:
	friend class TIntrusiveDoubleLinkedListIterator<TIntrusiveDoubleLinkedListNode>;
	friend class TIntrusiveDoubleLinkedList<ElementType, ContainerType>;

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType*       GetThisElement()       { return static_cast<ElementType*>(this); }
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType* GetThisElement() const { return static_cast<const ElementType*>(this); }

	ElementType* Next;
	ElementType* Prev;
};  // TIntrusiveDoubleLinkedListNode


/**
 * Iterator for intrusive double linked list.
 */
template <class NodeType>
class TIntrusiveDoubleLinkedListIterator
{
public:
	using ElementType = typename NodeType::ElementType;

	[[nodiscard]] UE_FORCEINLINE_HINT explicit TIntrusiveDoubleLinkedListIterator(ElementType* Node)
		: CurrentNode(Node)
	{
	}

	inline TIntrusiveDoubleLinkedListIterator& operator++()
	{
		checkSlow(CurrentNode);
		CurrentNode = CurrentNode->NodeType::Next;
		return *this;
	}

	inline TIntrusiveDoubleLinkedListIterator operator++(int)
	{
		auto Tmp = *this;
		++(*this);
		return Tmp;
	}

	inline TIntrusiveDoubleLinkedListIterator& operator--()
	{
		checkSlow(CurrentNode);
		CurrentNode = CurrentNode->NodeType::Prev;
		return *this;
	}

	inline TIntrusiveDoubleLinkedListIterator operator--(int)
	{
		auto Tmp = *this;
		--(*this);
		return Tmp;
	}

	// Accessors.
	[[nodiscard]] inline ElementType& operator->() const
	{
		checkSlow(CurrentNode);
		return *CurrentNode;
	}

	[[nodiscard]] inline ElementType& operator*() const
	{
		checkSlow(CurrentNode);
		return *CurrentNode;
	}

	[[nodiscard]] inline ElementType* GetNode() const
	{
		checkSlow(CurrentNode);
		return CurrentNode;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TIntrusiveDoubleLinkedListIterator& Other) const { return CurrentNode == Other.CurrentNode; }
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TIntrusiveDoubleLinkedListIterator& Other) const { return CurrentNode != Other.CurrentNode; }

private:
	ElementType* CurrentNode;
};  // TIntrusiveDoubleLinkedListIterator


/**
 * Intrusive double linked list.
 *
 * @see TDoubleLinkedList
 */
template <class InElementType, class ContainerType = InElementType>
class TIntrusiveDoubleLinkedList
{
public:

	using ElementType = InElementType;
	using NodeType = TIntrusiveDoubleLinkedListNode<ElementType, ContainerType>;

	[[nodiscard]] UE_FORCEINLINE_HINT TIntrusiveDoubleLinkedList()
	{
	}

	UE_FORCEINLINE_HINT TIntrusiveDoubleLinkedList(const TIntrusiveDoubleLinkedList&) = delete;
	UE_FORCEINLINE_HINT TIntrusiveDoubleLinkedList& operator=(const TIntrusiveDoubleLinkedList&) = delete;

	TIntrusiveDoubleLinkedList(TIntrusiveDoubleLinkedList&& Other)
	{
		AddHead(Forward<TIntrusiveDoubleLinkedList&&>(Other));
	}

	/**
	 * Fast empty that clears this list *without* changing the links in any elements.
	 * 
	 * @see IsEmpty, IsFilled
	 */
	UE_FORCEINLINE_HINT void Reset()
	{
		Sentinel.Reset();
	}

	// Accessors.

	[[nodiscard]] UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return Sentinel.Next == GetSentinel();
	}
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsFilled() const
	{
		return Sentinel.Next != GetSentinel();
	}

	// Adding/Removing methods

	UE_FORCEINLINE_HINT void AddHead(ElementType* Element)
	{
		check(Element && !Element->IsInList());
		static_cast<NodeType*>(Element)->InsertAfter(GetSentinel());
	}

	inline void AddHead(TIntrusiveDoubleLinkedList&& Other)
	{
		if (Other.IsFilled())
		{
			static_cast<NodeType*>(Other.Sentinel.Prev)->Next = Sentinel.Next;
			static_cast<NodeType*>(Other.Sentinel.Next)->Prev = GetSentinel();
			static_cast<NodeType*>(Sentinel.Next)->Prev = Other.Sentinel.Prev;
			Sentinel.Next = Other.Sentinel.Next;
			Other.Sentinel.Next = Other.Sentinel.Prev = Other.GetSentinel();
		}
	}

	UE_FORCEINLINE_HINT void AddTail(ElementType* Element)
	{
		check(Element && !Element->IsInList());
		static_cast<NodeType*>(Element)->InsertBefore(GetSentinel());
	}

	inline void AddTail(TIntrusiveDoubleLinkedList&& Other)
	{
		if (Other.IsFilled())
		{
			static_cast<NodeType*>(Other.Sentinel.Next)->Prev = Sentinel.Prev;
			static_cast<NodeType*>(Other.Sentinel.Prev)->Next = GetSentinel();
			static_cast<NodeType*>(Sentinel.Prev)->Next = Other.Sentinel.Next;
			Sentinel.Prev = Other.Sentinel.Prev;
			Other.Sentinel.Next = Other.Sentinel.Prev = Other.GetSentinel();
		}
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType* GetHead()
	{
		return IsFilled() ? Sentinel.Next : nullptr;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType* GetTail()
	{
		return IsFilled() ? Sentinel.Prev : nullptr;
	}

	[[nodiscard]] inline ElementType* PopHead()
	{
		if (IsEmpty())
		{
			return nullptr;
		}

		ElementType* Head = Sentinel.Next;
		static_cast<NodeType*>(Head)->Remove();
		return Head;
	}

	[[nodiscard]] inline ElementType* PopTail()
	{
		if (IsEmpty())
		{
			return nullptr;
		}

		ElementType* Tail = Sentinel.Prev;
		static_cast<NodeType*>(Tail)->Remove();
		return Tail;
	}

	UE_FORCEINLINE_HINT static void Remove(ElementType* Element)
	{
		static_cast<NodeType*>(Element)->Remove();
	}

	UE_FORCEINLINE_HINT static void InsertAfter(ElementType* InsertThis, ElementType* AfterThis)
	{
		static_cast<NodeType*>(InsertThis)->InsertAfter(AfterThis);
	}

	UE_FORCEINLINE_HINT static void InsertBefore(ElementType* InsertThis, ElementType* BeforeThis)
	{
		static_cast<NodeType*>(InsertThis)->InsertBefore(BeforeThis);
	}

	using TIterator = TIntrusiveDoubleLinkedListIterator<NodeType>;
	using TConstIterator = TIntrusiveDoubleLinkedListIterator<const NodeType>;

	[[nodiscard]] UE_FORCEINLINE_HINT TIterator      begin()       { return TIterator(Sentinel.Next); }
	[[nodiscard]] UE_FORCEINLINE_HINT TConstIterator begin() const { return TConstIterator(Sentinel.Next); }
	[[nodiscard]] UE_FORCEINLINE_HINT TIterator      end()         { return TIterator(GetSentinel()); }
	[[nodiscard]] UE_FORCEINLINE_HINT TConstIterator end() const   { return TConstIterator(GetSentinel()); }

private:

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType*       GetSentinel()       { return static_cast<ElementType*>(&Sentinel); }
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType* GetSentinel() const { return static_cast<const ElementType*>(&Sentinel); }

	NodeType Sentinel;

};  // TIntrusiveDoubleLinkedList

