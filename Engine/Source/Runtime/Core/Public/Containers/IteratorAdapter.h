// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UnrealTemplate.h"

template <typename Base>
class TIteratorAdapter;

/**
 * Adapter class for iterator implementation.
 *
 * This class provides an interface to implement UE compatible iterators with a minimum set of implementation
 * requirements. This provides the user with an easier way to implement custom iterators. The adapter uses a CRTP
 * pattern requiring its base class to implement the following:
 *
 * Types:
 *   - ElementType: the element type used in the container.
 *   - SizeType: the type use for the container's element indexing and iterator's offset.
 * Methods:
 *   - bool          Equals(const IteratorType& other)
 *   - ElementType&  Dereference() const;
 *   - void          Increment();
 * Optional:
 *   - void          Decrement();
 */
template <typename Base>
class TIteratorAdapter : public Base
{
public:
	using BaseType = Base;
	using ThisType = TIteratorAdapter<Base>;
	using ElementType = typename BaseType::ElementType;
	using SizeType = typename BaseType::SizeType;

	[[nodiscard]] TIteratorAdapter() = default;

	/**
	 * Perfect forwarding constructor to the iterator class.
	 */
	template < typename... Args>
	[[nodiscard]] TIteratorAdapter(EInPlace, Args&&... InArgs) : BaseType(Forward<Args>(InArgs)...)
	{
	}

	[[nodiscard]] ElementType& operator*() const
	{
		return this->Dereference();
	}

	[[nodiscard]] ElementType* operator->() const
	{
		return &this->Dereference();
	}

	ThisType& operator++()
	{
		this->Increment();
		return *this;
	}

	ThisType operator++(int)
	{
		ThisType Temp = *this;
		++*this;
		return Temp;
	}

	ThisType& operator--()
	{
		this->Decrement();
		return *this;
	}

	ThisType operator--(int)
	{
		ThisType Temp = *this;
		--*this;
		return Temp;
	}

	ThisType& operator+=(SizeType Offset)
	{
		this->Increment(Offset);
		return *this;
	}

	ThisType& operator-=(SizeType Offset)
	{
		this->Decrement(Offset);
		return *this;
	}

	ThisType operator+(SizeType Offset) const
	{
		ThisType Temp = *this;
		return Temp += Offset;
	}

	ThisType operator-(SizeType Offset) const
	{
		ThisType Temp = *this;
		return Temp -= Offset;
	}

protected:
	// BaseType MUST implement at least forward iteration which is enforced with the "using" declaration.
	using BaseType::Dereference;
	using BaseType::Equals;
	using BaseType::Increment;

	template <typename AnyElementType>
	friend class TIteratorAdapter;

public:
	[[nodiscard]] bool operator==(const TIteratorAdapter& Right) const
	{
		return Equals(Right);
	}

	[[nodiscard]] bool operator!=(const TIteratorAdapter& Right) const
	{
		return !Equals(Right);
	}
};
