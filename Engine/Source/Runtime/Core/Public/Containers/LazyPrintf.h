// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/UnrealString.h"

// to avoid limits with the Printf parameter count, for more readability and type safety
class FLazyPrintf
{
public:
	// constructor
	FLazyPrintf(const TCHAR* InputWithPercentS)
		: CurrentInputPos(InputWithPercentS)
	{
		// to avoid reallocations
		CurrentState.Empty(50 * 1024);
	}

	[[nodiscard]] FString GetResultString()
	{
		// internal error more %s than %s in MaterialTemplate.usf
		check(!ProcessUntilPercentS());

		// copy all remaining input data
		CurrentState += CurrentInputPos;

		return CurrentState;
	}

	// %s
	void PushParam(const TCHAR* Data)
	{
		if(ProcessUntilPercentS())
		{
			CurrentState += Data;
		}
		else
		{
			// internal error, more ReplacePercentS() calls than %s in MaterialTemplate.ush
			check(0);
		}
	}

private:

	// @param Pattern e.g. TEXT("%s")
	bool ProcessUntilPercentS()
	{
		const TCHAR* Found = FCString::Strstr(CurrentInputPos, TEXT("%s"));

		if(Found == 0)
		{
			return false;
		}

		// copy from input until %s
		while(CurrentInputPos < Found)
		{
			// can cause reallocations we could avoid
			CurrentState += *CurrentInputPos++;
		}

		// jump over %s
		CurrentInputPos += 2;

		return true;
	}

	const TCHAR* CurrentInputPos;
	FString CurrentState;
};
