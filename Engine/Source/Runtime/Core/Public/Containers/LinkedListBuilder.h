// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"

template<typename InElementType>
struct TLinkedListBuilderNextLink
{
	using ElementType = InElementType;

	[[nodiscard]] UE_FORCEINLINE_HINT static ElementType** GetNextPtr(ElementType& Element)
	{
		return &(Element.Next);
	}
};

template<typename InElementType, InElementType* InElementType::* NextLink>
struct TLinkedListBuilderNextLinkMemberVar
{
	using ElementType = InElementType;

	[[nodiscard]] UE_FORCEINLINE_HINT static ElementType** GetNextPtr(ElementType& Element)
	{
		return &(Element.*NextLink);
	}
};

/**
 * Single linked list builder base
 */
template<typename InElementType, typename InPointerType, typename InLinkAccessor>
struct TLinkedListBuilderBase
{
public:
	using ElementType = InElementType;
	using PointerType = InPointerType;
	using LinkAccessor = InLinkAccessor;

	UE_NONCOPYABLE(TLinkedListBuilderBase);

private:

	inline void WriteEndPtr(PointerType NewValue)
	{
		// Do not overwrite the same value to avoid dirtying the cache and
		// also prevent TSAN from thinking we are messing around with existing data.
		if (*EndPtr != NewValue)
		{
			*EndPtr = NewValue;
		}
	}

public:

	[[nodiscard]] explicit TLinkedListBuilderBase(PointerType* ListStartPtr) :
		StartPtr(ListStartPtr),
		EndPtr(ListStartPtr)
	{
		check(ListStartPtr);
	}

	// Move builder back to start and prepare for overwriting
	// It only changes state of builder, use NullTerminate() to mark list as empty!
	UE_FORCEINLINE_HINT void Restart()
	{
		EndPtr = StartPtr;
	}

	// Append element, don't touch next link
	inline void AppendNoTerminate(ElementType& Element)
	{
		WriteEndPtr(&Element);
		EndPtr = LinkAccessor::GetNextPtr(Element);
	}

	// Append element and mark it as last
	inline void AppendTerminated(ElementType& Element)
	{
		AppendNoTerminate(Element);
		NullTerminate();
	}

private:

	// Helper method to remove the element pointed to by CurrentLinkPtr.
	// CurrentLinkPtr is either a pointer to the list start pointer or the 
	// next pointer in the previous element
	inline void RemoveImpl(PointerType* CurrentLinkPtr)
	{

		// This gets the address of the next pointer of the element being removed
		PointerType* NextLinkPtr = LinkAccessor::GetNextPtr(**CurrentLinkPtr);

		// If EndPtr points to the next pointer of the element being removed,
		// then move the EndPtr back one element so it remains valid.
		if (EndPtr == NextLinkPtr)
		{
			EndPtr = CurrentLinkPtr;
		}

		// Update previous element link to point to the next element after the element being removed
		*CurrentLinkPtr = *NextLinkPtr;

		// Clear the next pointer in the removed element
		*NextLinkPtr = nullptr;
	}

public:

	// Remove all instances that match the predicate.  Returns the number of elements removed.
	template <class PREDICATE_CLASS>
	inline int32 RemoveAll(const PREDICATE_CLASS& Predicate)
	{
		int32 Removed = 0;

		// With the single link list, CurrentLinkPtr will always point to the address of the
		// variable that points to the current element to be tested, not the element itself.
		for (PointerType* CurrentLinkPtr = StartPtr; *CurrentLinkPtr;)
		{
			if (Predicate(*CurrentLinkPtr))
			{
				RemoveImpl(CurrentLinkPtr);
				++Removed;
			}
			else
			{
				CurrentLinkPtr = LinkAccessor::GetNextPtr(**CurrentLinkPtr);
			}
		}

		return Removed;
	}

	inline void Remove(ElementType& Element)
	{
		// With the single link list, CurrentLinkPtr will always point to the address of the
		// variable that points to the current element to be tested, not the element itself.
		for (PointerType* CurrentLinkPtr = StartPtr; *CurrentLinkPtr; CurrentLinkPtr = LinkAccessor::GetNextPtr(**CurrentLinkPtr))
		{
			if (*CurrentLinkPtr == &Element)
			{
				RemoveImpl(CurrentLinkPtr);
				break;
			}
		}
	}

	// Mark end of the list
	UE_FORCEINLINE_HINT void NullTerminate()
	{
		WriteEndPtr(nullptr);
	}

	inline void MoveToEnd()
	{
		for (PointerType It = *StartPtr; It; It = GetNext(*It))
		{
			EndPtr = LinkAccessor::GetNextPtr(*It);
		}
	}

	inline bool MoveToNext()
	{
		if (*EndPtr)
		{
			EndPtr = LinkAccessor::GetNextPtr(**EndPtr);
			return true;
		}

		return false;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT PointerType GetNext(ElementType& Element) const
	{
		return GetNextRef(Element);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT PointerType GetListStart() const
	{
		return *StartPtr;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT PointerType GetListEnd() const
	{
		return *EndPtr;
	}

private:
	[[nodiscard]] UE_FORCEINLINE_HINT PointerType& GetNextRef(ElementType& Element) const
	{
		return *LinkAccessor::GetNextPtr(Element);
	}

	PointerType* StartPtr;
	PointerType* EndPtr;
};

/**
 * Single linked list builder for raw pointers
 */

template<typename InElementType, typename InLinkAccessor = TLinkedListBuilderNextLink<InElementType>>
struct TLinkedListBuilder : public TLinkedListBuilderBase<InElementType, InElementType*, InLinkAccessor>
{
	using Super = TLinkedListBuilderBase<InElementType, InElementType*, InLinkAccessor>;
	using Super::Super;

	UE_NONCOPYABLE(TLinkedListBuilder);
};
