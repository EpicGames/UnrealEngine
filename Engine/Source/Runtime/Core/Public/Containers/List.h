// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"

template <class ContainerType>
class TLinkedListIteratorBase
{
public:
	[[nodiscard]] explicit TLinkedListIteratorBase(ContainerType* FirstLink)
		: CurrentLink(FirstLink)
	{
	}

	/**
	 * Advances the iterator to the next element.
	 */
	inline void Next()
	{
		checkSlow(CurrentLink);
		CurrentLink = (ContainerType*)CurrentLink->GetNextLink();
	}

	inline TLinkedListIteratorBase& operator++()
	{
		Next();
		return *this;
	}

	inline TLinkedListIteratorBase operator++(int)
	{
		auto Tmp = *this;
		Next();
		return Tmp;
	}

	/** conversion to "bool" returning true if the iterator is valid. */
	[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
	{ 
		return CurrentLink != nullptr;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TLinkedListIteratorBase& Rhs) const { return CurrentLink == Rhs.CurrentLink; }
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TLinkedListIteratorBase& Rhs) const { return CurrentLink != Rhs.CurrentLink; }
protected:

	ContainerType* CurrentLink;
};

template <class ContainerType, class ElementType>
class TLinkedListIterator : public TLinkedListIteratorBase<ContainerType>
{
	typedef TLinkedListIteratorBase<ContainerType> Super;

public:
	[[nodiscard]] explicit TLinkedListIterator(ContainerType* FirstLink)
		: Super(FirstLink)
	{
	}

	// Accessors.
	inline ElementType& operator->() const
	{
		checkSlow(this->CurrentLink);
		return **(this->CurrentLink);
	}

	inline ElementType& operator*() const
	{
		checkSlow(this->CurrentLink);
		return **(this->CurrentLink);
	}
};

template <class ContainerType, class ElementType>
class TIntrusiveLinkedListIterator : public TLinkedListIteratorBase<ElementType>
{
	typedef TLinkedListIteratorBase<ElementType> Super;

public:
	[[nodiscard]] explicit TIntrusiveLinkedListIterator(ElementType* FirstLink)
		: Super(FirstLink)
	{
	}

	// Accessors.
	inline ElementType& operator->() const
	{
		checkSlow(this->CurrentLink);
		return *(this->CurrentLink);
	}

	inline ElementType& operator*() const
	{
		checkSlow(this->CurrentLink);
		return *(this->CurrentLink);
	}
};


/**
 * Base linked list class, used to implement methods shared by intrusive/non-intrusive linked lists
 */
template <class ContainerType, class ElementType, template<class, class> class IteratorType>
class TLinkedListBase
{
public:
	/**
	 * Used to iterate over the elements of a linked list.
	 */
	typedef IteratorType<ContainerType,       ElementType> TIterator;
	typedef IteratorType<ContainerType, const ElementType> TConstIterator;


	/**
	 * Default constructor (empty list)
	 */
	[[nodiscard]] TLinkedListBase()
		: NextLink(NULL)
		, PrevLink(NULL)
	{
	}

	/**
	 * Removes this element from the list in constant time.
	 *
	 * This function is safe to call even if the element is not linked.
	 */
	inline void Unlink()
	{
		if( NextLink )
		{
			NextLink->PrevLink = PrevLink;
		}
		if( PrevLink )
		{
			*PrevLink = NextLink;
		}
		// Make it safe to call Unlink again.
		NextLink = nullptr;
		PrevLink = nullptr;
	}


	/**
	 * Adds this element to a list, before the given element.
	 *
	 * @param Before	The link to insert this element before.
	 */
	inline void LinkBefore(ContainerType* Before)
	{
		checkSlow(Before != NULL);

		PrevLink = Before->PrevLink;
		Before->PrevLink = &NextLink;

		NextLink = Before;

		if (PrevLink != NULL)
		{
			*PrevLink = (ContainerType*)this;
		}
	}

	/**
	 * Adds this element to the linked list, after the specified element
	 *
	 * @param After		The link to insert this element after.
	 */
	inline void LinkAfter(ContainerType* After)
	{
		checkSlow(After != NULL);

		PrevLink = &After->NextLink;
		NextLink = *PrevLink;
		*PrevLink = (ContainerType*)this;

		if (NextLink != NULL)
		{
			NextLink->PrevLink = &NextLink;
		}
	}

	/**
	 * Adds this element to the linked list, replacing the specified element.
	 * This is equivalent to calling LinkBefore(Replace); Replace->Unlink();
	 *
	 * @param Replace	Pointer to the element to be replaced
	 */
	inline void LinkReplace(ContainerType* Replace)
	{
		checkSlow(Replace != NULL);

		ContainerType**& ReplacePrev = Replace->PrevLink;
		ContainerType*& ReplaceNext = Replace->NextLink;

		PrevLink = ReplacePrev;
		NextLink = ReplaceNext;

		if (PrevLink != NULL)
		{
			*PrevLink = (ContainerType*)this;
		}

		if (NextLink != NULL)
		{
			NextLink->PrevLink = &NextLink;
		}

		ReplacePrev = NULL;
		ReplaceNext = NULL;
	}


	/**
	 * Adds this element as the head of the linked list, linking the input Head pointer to this element,
	 * so that when the element is linked/unlinked, the Head linked list pointer will be correctly updated.
	 *
	 * If Head already has an element, this functions like LinkBefore.
	 *
	 * @param Head		Pointer to the head of the linked list - this pointer should be the main reference point for the linked list
	 */
	inline void LinkHead(ContainerType*& Head)
	{
		if (Head != NULL)
		{
			Head->PrevLink = &NextLink;
		}

		NextLink = Head;
		PrevLink = &Head;
		Head = (ContainerType*)this;
	}


	/**
	 * Returns whether element is currently linked.
	 *
	 * @return true if currently linked, false otherwise
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsLinked()
	{
		return PrevLink != nullptr;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ContainerType** GetPrevLink() const
	{
		return PrevLink;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ContainerType* GetNextLink() const
	{
		return NextLink;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ContainerType* Next()
	{
		return NextLink;
	}

private:
	/** The next link in the linked list */
	ContainerType*  NextLink;

	/** Pointer to 'NextLink', within the previous link in the linked list */
	ContainerType** PrevLink;


	[[nodiscard]] UE_FORCEINLINE_HINT friend TIterator      begin(      ContainerType& List) { return TIterator     (&List); }
	[[nodiscard]] UE_FORCEINLINE_HINT friend TConstIterator begin(const ContainerType& List) { return TConstIterator(const_cast<ContainerType*>(&List)); }
	[[nodiscard]] UE_FORCEINLINE_HINT friend TIterator      end  (      ContainerType& List) { return TIterator     (nullptr); }
	[[nodiscard]] UE_FORCEINLINE_HINT friend TConstIterator end  (const ContainerType& List) { return TConstIterator(nullptr); }
};

/**
 * Encapsulates a link in a single linked list with constant access time.
 *
 * This linked list is non-intrusive, i.e. it stores a copy of the element passed to it (typically a pointer)
 */
template <class ElementType>
class TLinkedList : public TLinkedListBase<TLinkedList<ElementType>, ElementType, TLinkedListIterator>
{
	typedef TLinkedListBase<TLinkedList<ElementType>, ElementType, TLinkedListIterator>		Super;

public:
	/** Default constructor (empty list). */
	[[nodiscard]] TLinkedList()
		: Super()
	{
	}

	/**
	 * Creates a new linked list with a single element.
	 *
	 * @param InElement The element to add to the list.
	 */
	[[nodiscard]] explicit TLinkedList(const ElementType& InElement)
		: Super()
		, Element(InElement)
	{
	}


	// Accessors.
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType* operator->()
	{
		return &Element;
	}
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType* operator->() const
	{
		return &Element;
	}
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& operator*()
	{
		return Element;
	}
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType& operator*() const
	{
		return Element;
	}


private:
	ElementType   Element;
};

/**
 * Encapsulates a link in a single linked list with constant access time.
 * Structs/classes must inherit this, to use it, e.g: struct FMyStruct : public TIntrusiveLinkedList<FMyStruct>
 *
 * This linked list is intrusive, i.e. the element is a subclass of this link, so that each link IS the element.
 *
 * Never reference TIntrusiveLinkedList outside of the above class/struct inheritance, only ever refer to the struct, e.g:
 *	FMyStruct* MyLinkedList = NULL;
 *
 *	FMyStruct* StructLink = new FMyStruct();
 *	StructLink->LinkHead(MyLinkedList);
 *
 *	for (FMyStruct::TIterator It(MyLinkedList); It; It.Next())
 *	{
 *		...
 *	}
 */
template <class ElementType>
class TIntrusiveLinkedList : public TLinkedListBase<ElementType, ElementType, TIntrusiveLinkedListIterator>
{
	typedef TLinkedListBase<ElementType, ElementType, TIntrusiveLinkedListIterator>		Super;

public:
	/** Default constructor (empty list). */
	[[nodiscard]] TIntrusiveLinkedList()
		: Super()
	{
	}
};


template <class NodeType, class ElementType>
class TDoubleLinkedListIterator
{
public:

	[[nodiscard]] explicit TDoubleLinkedListIterator(NodeType* StartingNode)
		: CurrentNode(StartingNode)
	{
	}

	/** conversion to "bool" returning true if the iterator is valid. */
	[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
	{ 
		return CurrentNode != nullptr; 
	}

	TDoubleLinkedListIterator& operator++()
	{
		checkSlow(CurrentNode);
		CurrentNode = CurrentNode->GetNextNode();
		return *this;
	}

	TDoubleLinkedListIterator operator++(int)
	{
		auto Tmp = *this;
		++(*this);
		return Tmp;
	}

	TDoubleLinkedListIterator& operator--()
	{
		checkSlow(CurrentNode);
		CurrentNode = CurrentNode->GetPrevNode();
		return *this;
	}

	TDoubleLinkedListIterator operator--(int)
	{
		auto Tmp = *this;
		--(*this);
		return Tmp;
	}

	// Accessors.
	[[nodiscard]] ElementType& operator->() const
	{
		checkSlow(CurrentNode);
		return CurrentNode->GetValue();
	}

	[[nodiscard]] ElementType& operator*() const
	{
		checkSlow(CurrentNode);
		return CurrentNode->GetValue();
	}

	[[nodiscard]] NodeType* GetNode() const
	{
		checkSlow(CurrentNode);
		return CurrentNode;
	}

	[[nodiscard]] bool operator==(const TDoubleLinkedListIterator& Rhs) const { return CurrentNode == Rhs.CurrentNode; }
	[[nodiscard]] bool operator!=(const TDoubleLinkedListIterator& Rhs) const { return CurrentNode != Rhs.CurrentNode; }

private:
	NodeType* CurrentNode;
};


/**
 * Double linked list.
 * 
 * @see TIntrusiveDoubleLinkedList
 */
template <class ElementType>
class TDoubleLinkedList
{
public:
	class TDoubleLinkedListNode
	{
	public:
		friend class TDoubleLinkedList;

		/** Constructor */
		[[nodiscard]] TDoubleLinkedListNode( const ElementType& InValue )
			: Value(InValue), NextNode(nullptr), PrevNode(nullptr)
		{
		}

		[[nodiscard]] const ElementType& GetValue() const
		{
			return Value;
		}

		[[nodiscard]] ElementType& GetValue()
		{
			return Value;
		}

		[[nodiscard]] TDoubleLinkedListNode* GetNextNode()
		{
			return NextNode;
		}

		[[nodiscard]] const TDoubleLinkedListNode* GetNextNode() const
		{
			return NextNode;
		}

		[[nodiscard]] TDoubleLinkedListNode* GetPrevNode()
		{
			return PrevNode;
		}

		[[nodiscard]] const TDoubleLinkedListNode* GetPrevNode() const
		{
			return PrevNode;
		}

	protected:
		ElementType            Value;
		TDoubleLinkedListNode* NextNode;
		TDoubleLinkedListNode* PrevNode;
	};

	/**
	 * Used to iterate over the elements of a linked list.
	 */
	typedef TDoubleLinkedListIterator<TDoubleLinkedListNode,       ElementType> TIterator;
	typedef TDoubleLinkedListIterator<TDoubleLinkedListNode, const ElementType> TConstIterator;

	/** Constructors. */
	[[nodiscard]] TDoubleLinkedList()
		: HeadNode(nullptr)
		, TailNode(nullptr)
		, ListSize(0)
	{
	}

	/** Destructor */
	virtual ~TDoubleLinkedList()
	{
		Empty();
	}

	// Can't copy
	TDoubleLinkedList(const TDoubleLinkedList&) = delete;
	TDoubleLinkedList& operator=(const TDoubleLinkedList&) = delete;

	/** Move Constructor/Assignment */
	[[nodiscard]] TDoubleLinkedList(TDoubleLinkedList&& Other)
		: TDoubleLinkedList()
	{
		Swap(HeadNode, Other.HeadNode);
		Swap(TailNode, Other.TailNode);
		Swap(ListSize, Other.ListSize);
	}
	
	TDoubleLinkedList& operator=(TDoubleLinkedList&& Other)
	{
		Swap(HeadNode, Other.HeadNode);
		Swap(TailNode, Other.TailNode);
		Swap(ListSize, Other.ListSize);
		return *this;
	}

	// Adding/Removing methods

	/**
	 * Add the specified value to the beginning of the list, making that value the new head of the list.
	 *
	 * @param	InElement	the value to add to the list.
	 * @return	whether the node was successfully added into the list.
	 * @see GetHead, InsertNode, RemoveNode
	 */
	bool AddHead( const ElementType& InElement )
	{
		return AddHead(new TDoubleLinkedListNode(InElement));
	}

	bool AddHead( TDoubleLinkedListNode* NewNode )
	{
		if (NewNode == nullptr)
		{
			return false;
		}

		// have an existing head node - change the head node to point to this one
		if ( HeadNode != nullptr )
		{
			NewNode->NextNode = HeadNode;
			HeadNode->PrevNode = NewNode;
			HeadNode = NewNode;
		}
		else
		{
			HeadNode = TailNode = NewNode;
		}

		SetListSize(ListSize + 1);
		return true;
	}

	/**
	 * Append the specified value to the end of the list
	 *
	 * @param	InElement	the value to add to the list.
	 * @return	whether the node was successfully added into the list
	 * @see GetTail, InsertNode, RemoveNode
	 */
	bool AddTail( const ElementType& InElement )
	{
		return AddTail(new TDoubleLinkedListNode(InElement));
	}

	bool AddTail( TDoubleLinkedListNode* NewNode )
	{
		if ( NewNode == nullptr )
		{
			return false;
		}

		if ( TailNode != nullptr )
		{
			TailNode->NextNode = NewNode;
			NewNode->PrevNode = TailNode;
			TailNode = NewNode;
		}
		else
		{
			HeadNode = TailNode = NewNode;
		}

		SetListSize(ListSize + 1);
		return true;
	}

	/**
	 * Insert the specified value into the list at an arbitrary point.
	 *
	 * @param	InElement			the value to insert into the list
	 * @param	NodeToInsertBefore	the new node will be inserted into the list at the current location of this node
	 *								if nullptr, the new node will become the new head of the list
	 * @return	whether the node was successfully added into the list
	 * @see Empty, RemoveNode
	 */
	bool InsertNode( const ElementType& InElement, TDoubleLinkedListNode* NodeToInsertBefore=nullptr )
	{
		return InsertNode(new TDoubleLinkedListNode(InElement), NodeToInsertBefore);
	}

	bool InsertNode( TDoubleLinkedListNode* NewNode, TDoubleLinkedListNode* NodeToInsertBefore=nullptr )
	{
		if ( NewNode == nullptr )
		{
			return false;
		}

		if ( NodeToInsertBefore == nullptr || NodeToInsertBefore == HeadNode )
		{
			return AddHead(NewNode);
		}

		NewNode->PrevNode = NodeToInsertBefore->PrevNode;
		NewNode->NextNode = NodeToInsertBefore;

		NodeToInsertBefore->PrevNode->NextNode = NewNode;
		NodeToInsertBefore->PrevNode = NewNode;

		SetListSize(ListSize + 1);
		return true;
	}

	/**
	 * Remove the node corresponding to InElement.
	 *
	 * @param InElement The value to remove from the list.
	 * @see Empty, InsertNode
	 */
	void RemoveNode( const ElementType& InElement )
	{
		TDoubleLinkedListNode* ExistingNode = FindNode(InElement);
		RemoveNode(ExistingNode);
	}

	/**
	 * Removes the node specified.
	 *
	 * @param NodeToRemove The node to remove.
	 * @see Empty, InsertNode
	 */
	void RemoveNode( TDoubleLinkedListNode* NodeToRemove, bool bDeleteNode = true )
	{
		if ( NodeToRemove != nullptr )
		{
			// if we only have one node, just call Clear() so that we don't have to do lots of extra checks in the code below
			if ( Num() == 1 )
			{
				checkSlow(NodeToRemove == HeadNode);
				if (bDeleteNode)
				{
					Empty();
				}
				else
				{
					NodeToRemove->NextNode = NodeToRemove->PrevNode = nullptr;
					HeadNode = TailNode = nullptr;
					SetListSize(0);
				}
				return;
			}

			if ( NodeToRemove == HeadNode )
			{
				HeadNode = HeadNode->NextNode;
				HeadNode->PrevNode = nullptr;
			}

			else if ( NodeToRemove == TailNode )
			{
				TailNode = TailNode->PrevNode;
				TailNode->NextNode = nullptr;
			}
			else
			{
				NodeToRemove->NextNode->PrevNode = NodeToRemove->PrevNode;
				NodeToRemove->PrevNode->NextNode = NodeToRemove->NextNode;
			}

			if (bDeleteNode)
			{
				delete NodeToRemove;
			}
			else
			{
				NodeToRemove->NextNode = NodeToRemove->PrevNode = nullptr;
			}
			SetListSize(ListSize - 1);
		}
	}

	/** Removes all nodes from the list. */
	void Empty()
	{
		TDoubleLinkedListNode* Node;
		while ( HeadNode != nullptr )
		{
			Node = HeadNode->NextNode;
			delete HeadNode;
			HeadNode = Node;
		}

		HeadNode = TailNode = nullptr;
		SetListSize(0);
	}

	// Accessors.

	/**
	 * Returns the node at the head of the list.
	 *
	 * @return Pointer to the first node.
	 * @see GetTail
	 */
	[[nodiscard]] TDoubleLinkedListNode* GetHead() const
	{
		return HeadNode;
	}

	/**
	 * Returns the node at the end of the list.
	 *
	 * @return Pointer to the last node.
	 * @see GetHead
	 */
	[[nodiscard]] TDoubleLinkedListNode* GetTail() const
	{
		return TailNode;
	}

	/**
	 * Finds the node corresponding to the value specified
	 *
	 * @param	InElement	the value to find
	 * @return	a pointer to the node that contains the value specified, or nullptr of the value couldn't be found
	 */
	[[nodiscard]] TDoubleLinkedListNode* FindNode( const ElementType& InElement )
	{
		TDoubleLinkedListNode* Node = HeadNode;
		while ( Node != nullptr )
		{
			if ( Node->GetValue() == InElement )
			{
				break;
			}

			Node = Node->NextNode;
		}

		return Node;
	}

	[[nodiscard]] bool Contains( const ElementType& InElement )
	{
		return (FindNode(InElement) != nullptr);
	}

	/**
	 * Returns true if the list is empty and contains no elements. 
	 *
	 * @returns True if the list is empty.
	 * @see Num
	 */
	[[nodiscard]] bool IsEmpty() const
	{
		return ListSize == 0;
	}

	/**
	 * Returns the number of items in the list.
	 *
	 * @return Item count.
	 */
	[[nodiscard]] int32 Num() const
	{
		return ListSize;
	}

protected:

	/**
	 * Updates the size reported by Num().  Child classes can use this function to conveniently
	 * hook into list additions/removals.
	 *
	 * @param	NewListSize		the new size for this list
	 */
	virtual void SetListSize( int32 NewListSize )
	{
		ListSize = NewListSize;
	}

private:
	TDoubleLinkedListNode* HeadNode;
	TDoubleLinkedListNode* TailNode;
	int32 ListSize;

	[[nodiscard]] friend TIterator      begin(      TDoubleLinkedList& List) { return TIterator     (List.GetHead()); }
	[[nodiscard]] friend TConstIterator begin(const TDoubleLinkedList& List) { return TConstIterator(List.GetHead()); }
	[[nodiscard]] friend TIterator      end  (      TDoubleLinkedList& List) { return TIterator     (nullptr); }
	[[nodiscard]] friend TConstIterator end  (const TDoubleLinkedList& List) { return TConstIterator(nullptr); }
};


/*----------------------------------------------------------------------------
	TList.
----------------------------------------------------------------------------*/

//
// Simple single-linked list template.
//
template <class ElementType> class TList
{
public:

	ElementType			Element;
	TList<ElementType>*	Next;

	// Constructor.

	[[nodiscard]] TList(const ElementType &InElement, TList<ElementType>* InNext = nullptr)
	{
		Element = InElement;
		Next = InNext;
	}
};
