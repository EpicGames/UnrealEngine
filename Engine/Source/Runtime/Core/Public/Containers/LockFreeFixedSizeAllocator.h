// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Misc/NoopCounter.h"
#include "Containers/LockFreeList.h"

/**
* Thread safe, lock free pooling allocator of fixed size blocks that
* never returns free space, even at shutdown.
* Alignment isn't handled; assumes FMemory::Malloc will work.
*/

#ifndef USE_NAIVE_TLockFreeFixedSizeAllocator_TLSCacheBase
#define USE_NAIVE_TLockFreeFixedSizeAllocator_TLSCacheBase (0) // this is useful for find who really leaked
#endif

template<int32 SIZE, typename TBundleRecycler, typename TTrackingCounter = FNoopCounter>
class TLockFreeFixedSizeAllocator_TLSCacheBase : public FNoncopyable
{
	enum
	{
		SIZE_PER_BUNDLE = 65536,
		NUM_PER_BUNDLE = SIZE_PER_BUNDLE / SIZE
	};

public:

	TLockFreeFixedSizeAllocator_TLSCacheBase()
	{
		static_assert(SIZE >= sizeof(void*) && SIZE % sizeof(void*) == 0, "Blocks in TLockFreeFixedSizeAllocator must be at least the size of a pointer.");
		TlsSlot = FPlatformTLS::AllocTlsSlot();
		check(FPlatformTLS::IsValidTlsSlot(TlsSlot));
	}
	/** Destructor, leaks all of the memory **/
	~TLockFreeFixedSizeAllocator_TLSCacheBase()
	{
		FPlatformTLS::FreeTlsSlot(TlsSlot);
		TlsSlot = FPlatformTLS::InvalidTlsSlot;
	}

	/**
	* Allocates a memory block of size SIZE.
	*
	* @return Pointer to the allocated memory.
	* @see Free
	*/
	inline void* Allocate()
	{
#if USE_NAIVE_TLockFreeFixedSizeAllocator_TLSCacheBase
		return FMemory::Malloc(SIZE);
#else
		FThreadLocalCache& TLS = GetTLS();

		if (!TLS.PartialBundle)
		{
			if (TLS.FullBundle)
			{
				TLS.PartialBundle = TLS.FullBundle;
				TLS.FullBundle = nullptr;
			}
			else
			{
				TLS.PartialBundle = GlobalFreeListBundles.Pop();
				if (!TLS.PartialBundle)
				{
					TLS.PartialBundle = (void**)FMemory::Malloc(SIZE_PER_BUNDLE);
					void **Next = TLS.PartialBundle;
					for (int32 Index = 0; Index < NUM_PER_BUNDLE - 1; Index++)
					{
						void* NextNext = (void*)(((uint8*)Next) + SIZE);
						*Next = NextNext;
						Next = (void**)NextNext;
					}
					*Next = nullptr;
					NumFree.Add(NUM_PER_BUNDLE);
				}
			}
			TLS.NumPartial = NUM_PER_BUNDLE;
		}
		NumUsed.Increment();
		NumFree.Decrement();
		void* Result = (void*)TLS.PartialBundle;
		TLS.PartialBundle = (void**)*TLS.PartialBundle;
		TLS.NumPartial--;
		check(TLS.NumPartial >= 0 && ((!!TLS.NumPartial) == (!!TLS.PartialBundle)));
		return Result;
#endif
	}

	/**
	* Puts a memory block previously obtained from Allocate() back on the free list for future use.
	*
	* @param Item The item to free.
	* @see Allocate
	*/
	inline void Free(void *Item)
	{
#if USE_NAIVE_TLockFreeFixedSizeAllocator_TLSCacheBase
		return FMemory::Free(Item);
#else
		NumUsed.Decrement();
		NumFree.Increment();
		FThreadLocalCache& TLS = GetTLS();
		if (TLS.NumPartial >= NUM_PER_BUNDLE)
		{
			if (TLS.FullBundle)
			{
				GlobalFreeListBundles.Push(TLS.FullBundle);
				//TLS.FullBundle = nullptr;
			}
			TLS.FullBundle = TLS.PartialBundle;
			TLS.PartialBundle = nullptr;
			TLS.NumPartial = 0;
		}
		*(void**)Item = (void*)TLS.PartialBundle;
		TLS.PartialBundle = (void**)Item;
		TLS.NumPartial++;
#endif
	}

	/**
	* Gets the number of allocated memory blocks that are currently in use.
	*
	* @return Number of used memory blocks.
	* @see GetNumFree
	*/
	const TTrackingCounter& GetNumUsed() const
	{
		return NumUsed;
	}

	/**
	* Gets the number of allocated memory blocks that are currently unused.
	*
	* @return Number of unused memory blocks.
	* @see GetNumUsed
	*/
	const TTrackingCounter& GetNumFree() const
	{
		return NumFree;
	}

private:

	/** struct for the TLS cache. */
	struct FThreadLocalCache
	{
		void **FullBundle;
		void **PartialBundle;
		int32 NumPartial;

		FThreadLocalCache()
			: FullBundle(nullptr)
			, PartialBundle(nullptr)
			, NumPartial(0)
		{
		}
	};

	FThreadLocalCache& GetTLS()
	{
		checkSlow(FPlatformTLS::IsValidTlsSlot(TlsSlot));
		FThreadLocalCache* TLS = (FThreadLocalCache*)FPlatformTLS::GetTlsValue(TlsSlot);
		if (!TLS)
		{
			TLS = new FThreadLocalCache();
			FPlatformTLS::SetTlsValue(TlsSlot, TLS);
		}
		return *TLS;
	}

	/** Slot for TLS struct. */
	uint32 TlsSlot;

	/** Lock free list of free memory blocks, these are all linked into a bundle of NUM_PER_BUNDLE. */
	TBundleRecycler GlobalFreeListBundles;

	/** Total number of blocks outstanding and not in the free list. */
	TTrackingCounter NumUsed;

	/** Total number of blocks in the free list. */
	TTrackingCounter NumFree;
};

/**
 * Thread safe, lock free pooling allocator of fixed size blocks that
 * only returns free space when the allocator is destroyed.
 * Alignment isn't handled; assumes FMemory::Malloc will work.
 */
template<int32 SIZE, int TPaddingForCacheContention, typename TTrackingCounter = FNoopCounter>
class TLockFreeFixedSizeAllocator
{
public:
	TLockFreeFixedSizeAllocator()
	{
		checkf(!AutoRTFM::IsClosed() || !AutoRTFM::IsOnCurrentTransactionStack(this), TEXT("Not allowed to construct a stack local within a transaction."));
	}

	/** Destructor, returns all memory via FMemory::Free **/
	~TLockFreeFixedSizeAllocator()
	{
		AutoRTFM::PopAllOnAbortHandlers(this);
		UE_AUTORTFM_ONCOMMIT(this)
		{
			check(GetNumUsed() == 0);
			Trim();
		};
	}

	/**
	 * Allocates a memory block of size SIZE.
	 *
	 * @return Pointer to the allocated memory.
	 * @see Free
	 */
	void* Allocate(int32 Alignment = MIN_ALIGNMENT)
	{
		void* Memory = nullptr;

		// We need the allocation to happen (so we get a valid `Memory`)
		// pointer to use in the transaction. So we do the actual meat
		// of the allocation in the open.
		UE_AUTORTFM_OPEN
		{
			NumUsed.Increment();
			if (Alignment <= 4096)
			{
				// Pop from a free list only if Alignment is not large than a memory page size
				Memory = FreeList.Pop();
				if (Memory)
				{
					NumFree.Decrement();
				}
			}
			if (!Memory)
			{
				Memory = FMemory::Malloc(SIZE, Alignment);
			}
		};

		// But if we abort the transaction, we need to return the memory
		// region to the allocator (otherwise we'll leak).
		UE_AUTORTFM_ONABORT(this, Memory)
		{
			this->Free(Memory);
		};
		
		return Memory;
	}

	/**
	 * Puts a memory block previously obtained from Allocate() back on the free list for future use.
	 *
	 * @param Item The item to free.
	 * @see Allocate
	 */
	void Free(void* Item)
	{
		// We defer actually returning `Item` until on-commit time.
		// If we didn't do this an aborting transaction would not
		// be able to be undone.
		UE_AUTORTFM_ONCOMMIT(this, Item)
		{
			NumUsed.Decrement();
			FreeList.Push(Item);
			NumFree.Increment();
		};
	}

	/**
	* Returns all free memory to the heap.
	*/
	void Trim()
	{
		// Similar to `Free`, we need to only do the destructive
		// trim operation at commit time - otherwise we'll wouldn't
		// be able to undo it.
		UE_AUTORTFM_ONCOMMIT(this)
		{
			while (void* Mem = FreeList.Pop())
			{
				FMemory::Free(Mem);
				NumFree.Decrement();
			}
		};
	}

	/**
	 * Gets the number of allocated memory blocks that are currently in use.
	 *
	 * @return Number of used memory blocks.
	 * @see GetNumFree
	 */
	typename TTrackingCounter::IntegerType GetNumUsed() const
	{
		return AutoRTFM::Open([&] AUTORTFM_DISABLE { return NumUsed.GetValue(); });
	}

	/**
	 * Gets the number of allocated memory blocks that are currently unused.
	 *
	 * @return Number of unused memory blocks.
	 * @see GetNumUsed
	 */
	typename TTrackingCounter::IntegerType GetNumFree() const
	{
		return AutoRTFM::Open([&] AUTORTFM_DISABLE { return NumFree.GetValue(); });
	}

private:
	UE_NONCOPYABLE(TLockFreeFixedSizeAllocator)

	/** Lock free list of free memory blocks. */
	TLockFreePointerListUnordered<void, TPaddingForCacheContention> FreeList;

	/** Total number of blocks outstanding and not in the free list. */
	TTrackingCounter NumUsed;

	/** Total number of blocks in the free list. */
	TTrackingCounter NumFree;
};

/**
 * Thread safe, lock free pooling allocator of fixed size blocks that
 * never returns free space, even at shutdown
 * alignment isn't handled, assumes FMemory::Malloc will work
 */
template<int32 SIZE, int TPaddingForCacheContention, typename TTrackingCounter = FNoopCounter>
class TLockFreeFixedSizeAllocator_TLSCache : public TLockFreeFixedSizeAllocator_TLSCacheBase<SIZE, TLockFreePointerListUnordered<void*, TPaddingForCacheContention>, TTrackingCounter>
{
};

/**
 * Thread safe, lock free pooling allocator of memory for instances of T.
 *
 * Never returns free space until program shutdown.
 */
template<class T, int TPaddingForCacheContention>
class TLockFreeClassAllocator : private TLockFreeFixedSizeAllocator<sizeof(T), TPaddingForCacheContention, FNoopCounter>
{
public:
	/**
	 * Returns a memory block of size sizeof(T).
	 *
	 * @return Pointer to the allocated memory.
	 * @see Free, New
	 */
	void* Allocate()
	{
		return TLockFreeFixedSizeAllocator<sizeof(T), TPaddingForCacheContention>::Allocate();
	}

	/**
	 * Returns a new T using the default constructor.
	 *
	 * @return Pointer to the new object.
	 * @see Allocate, Free
	 */
	T* New()
	{
		return ::new (Allocate()) T();
	}

	/**
	 * Calls a destructor on Item and returns the memory to the free list for recycling.
	 *
	 * @param Item The item whose memory to free.
	 * @see Allocate, New
	 */
	void Free(T *Item)
	{
		Item->~T();
		TLockFreeFixedSizeAllocator<sizeof(T), TPaddingForCacheContention>::Free(Item);
	}
};

/**
 * Thread safe, lock free pooling allocator of memory for instances of T.
 *
 * Never returns free space until program shutdown.
 */
template<class T, int TPaddingForCacheContention>
class TLockFreeClassAllocator_TLSCache : private TLockFreeFixedSizeAllocator_TLSCache<sizeof(T), TPaddingForCacheContention, FNoopCounter>
{
public:
	/**
	 * Returns a memory block of size sizeof(T).
	 *
	 * @return Pointer to the allocated memory.
	 * @see Free, New
	 */
	void* Allocate()
	{
		return TLockFreeFixedSizeAllocator_TLSCache<sizeof(T), TPaddingForCacheContention>::Allocate();
	}

	/**
	 * Returns a new T using the default constructor.
	 *
	 * @return Pointer to the new object.
	 * @see Allocate, Free
	 */
	T* New()
	{
		return ::new (Allocate()) T();
	}

	/**
	 * Calls a destructor on Item and returns the memory to the free list for recycling.
	 *
	 * @param Item The item whose memory to free.
	 * @see Allocate, New
	 */
	void Free(T *Item)
	{
		Item->~T();
		TLockFreeFixedSizeAllocator_TLSCache<sizeof(T), TPaddingForCacheContention>::Free(Item);
	}
};
