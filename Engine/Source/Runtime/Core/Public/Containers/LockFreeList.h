// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PlatformProcess.h"
#include "HAL/ThreadSafeCounter.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/NoopCounter.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/Function.h"

#include <atomic>

CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogLockFreeList, Log, All);

// what level of checking to perform...normally checkLockFreePointerList but could be ensure or check
#if 1
	#define checkLockFreePointerList checkSlow
#else
	#if PLATFORM_WINDOWS
		#pragma warning(disable : 4706)
	#endif
	#define checkLockFreePointerList(x) ((x)||((*(char*)3) = 0))
#endif

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST

	CORE_API void DoTestCriticalStall();
	extern CORE_API int32 GTestCriticalStalls;

	inline void TestCriticalStall()
	{
		if (GTestCriticalStalls)
		{
			DoTestCriticalStall();
		}
	}
#else
	UE_FORCEINLINE_HINT void TestCriticalStall()
	{
	}
#endif

CORE_API void LockFreeTagCounterHasOverflowed();
CORE_API void LockFreeLinksExhausted(uint32 TotalNum);
CORE_API void* LockFreeAllocLinks(SIZE_T AllocSize);
CORE_API void LockFreeFreeLinks(SIZE_T AllocSize, void* Ptr);

#define MAX_LOCK_FREE_LINKS_AS_BITS (26)
#define MAX_LOCK_FREE_LINKS (1 << 26)

template<class T, unsigned int MaxTotalItems, unsigned int ItemsPerPage>
class TLockFreeAllocOnceIndexedAllocator
{
	enum
	{
		MaxBlocks = (MaxTotalItems + ItemsPerPage - 1) / ItemsPerPage
	};
public:

	[[nodiscard]] TLockFreeAllocOnceIndexedAllocator()
	{
		NextIndex.Increment(); // skip the null ptr
		for (uint32 Index = 0; Index < MaxBlocks; Index++)
		{
			Pages[Index] = nullptr;
		}
	}

	inline uint32 Alloc(uint32 Count = 1)
	{
		uint32 FirstItem = NextIndex.Add(Count);
		if (FirstItem + Count > MaxTotalItems)
		{
			LockFreeLinksExhausted(MaxTotalItems);
		}
		for (uint32 CurrentItem = FirstItem; CurrentItem < FirstItem + Count; CurrentItem++)
		{
			::new (GetRawItem(CurrentItem)) T();
		}
		return FirstItem;
	}
	[[nodiscard]] inline T* GetItem(uint32 Index)
	{
		if (!Index)
		{
			return nullptr;
		}
		uint32 BlockIndex = Index / ItemsPerPage;
		uint32 SubIndex = Index % ItemsPerPage;
		checkLockFreePointerList(Index < (uint32)NextIndex.GetValue() && Index < MaxTotalItems && BlockIndex < MaxBlocks && Pages[BlockIndex].load(std::memory_order_relaxed));
		return Pages[BlockIndex].load(std::memory_order_relaxed) + SubIndex;
	}

private:
	[[nodiscard]] void* GetRawItem(uint32 Index)
	{
		uint32 BlockIndex = Index / ItemsPerPage;
		uint32 SubIndex = Index % ItemsPerPage;
		checkLockFreePointerList(Index && Index < (uint32)NextIndex.GetValue() && Index < MaxTotalItems && BlockIndex < MaxBlocks);

		if (!Pages[BlockIndex].load(std::memory_order_relaxed))
		{
			T* NewBlock = (T*)LockFreeAllocLinks(ItemsPerPage * sizeof(T));
			checkLockFreePointerList(IsAligned(NewBlock, alignof(T)));
			T* ExpectedValue = nullptr;
			if (!Pages[BlockIndex].compare_exchange_strong(ExpectedValue, NewBlock))
			{
				// we lost discard block
				checkLockFreePointerList(Pages[BlockIndex].load(std::memory_order_relaxed) && Pages[BlockIndex].load(std::memory_order_relaxed) != NewBlock);
				LockFreeFreeLinks(ItemsPerPage * sizeof(T), NewBlock);
			}
			else
			{
				checkLockFreePointerList(Pages[BlockIndex].load(std::memory_order_relaxed));
			}
		}
		return (void*)(Pages[BlockIndex].load(std::memory_order_relaxed) + SubIndex);
	}

	alignas(PLATFORM_CACHE_LINE_SIZE) FThreadSafeCounter NextIndex;
	alignas(PLATFORM_CACHE_LINE_SIZE) std::atomic<T*> Pages[MaxBlocks];
};


#define MAX_TagBitsValue (uint64(1) << (64 - MAX_LOCK_FREE_LINKS_AS_BITS))
struct FIndexedLockFreeLink;


MS_ALIGN(8)
struct FIndexedPointer
{
	// no constructor, intentionally. We need to keep the ABA double counter in tact

	// This should only be used for FIndexedPointer's with no outstanding concurrency.
	// Not recycled links, for example.
	void Init()
	{
		static_assert(((MAX_LOCK_FREE_LINKS - 1) & MAX_LOCK_FREE_LINKS) == 0, "MAX_LOCK_FREE_LINKS must be a power of two");
		Ptrs.store(0, std::memory_order_relaxed);
	}
	inline void SetAll(uint32 Ptr, uint64 CounterAndState)
	{
		checkLockFreePointerList(Ptr < MAX_LOCK_FREE_LINKS && CounterAndState < (uint64(1) << (64 - MAX_LOCK_FREE_LINKS_AS_BITS)));
		Ptrs.store(uint64(Ptr) | (CounterAndState << MAX_LOCK_FREE_LINKS_AS_BITS), std::memory_order_relaxed);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetPtr() const
	{
		return uint32(Ptrs.load(std::memory_order_relaxed) & (MAX_LOCK_FREE_LINKS - 1));
	}

	UE_FORCEINLINE_HINT void SetPtr(uint32 To)
	{
		SetAll(To, GetCounterAndState());
	}

	[[nodiscard]] UE_FORCEINLINE_HINT uint64 GetCounterAndState() const
	{
		return (Ptrs.load(std::memory_order_relaxed) >> MAX_LOCK_FREE_LINKS_AS_BITS);
	}

	UE_FORCEINLINE_HINT void SetCounterAndState(uint64 To)
	{
		SetAll(GetPtr(), To);
	}

	inline void AdvanceCounterAndState(const FIndexedPointer &From, uint64 TABAInc)
	{
		SetCounterAndState(From.GetCounterAndState() + TABAInc);
		if (UNLIKELY(GetCounterAndState() < From.GetCounterAndState()))
		{
			// this is not expected to be a problem and it is not expected to happen very often. When it does happen, we will sleep as an extra precaution.
			LockFreeTagCounterHasOverflowed();
		}
	}

	template<uint64 TABAInc>
	[[nodiscard]] UE_FORCEINLINE_HINT uint64 GetState() const
	{
		return GetCounterAndState() & (TABAInc - 1);
	}

	template<uint64 TABAInc>
	inline void SetState(uint64 Value)
	{
		checkLockFreePointerList(Value < TABAInc);
		SetCounterAndState((GetCounterAndState() & ~(TABAInc - 1)) | Value);
	}

	inline void AtomicRead(const FIndexedPointer& Other)
	{
		checkLockFreePointerList(IsAligned(&Ptrs, 8) && IsAligned(&Other.Ptrs, 8));
		Ptrs.store(Other.Ptrs.load(std::memory_order_acquire), std::memory_order_relaxed);
		TestCriticalStall();
	}

	inline bool InterlockedCompareExchange(const FIndexedPointer& Exchange, const FIndexedPointer& Comparand)
	{
		TestCriticalStall();
		uint64 Expected = Comparand.Ptrs.load(std::memory_order_relaxed);
		return Ptrs.compare_exchange_strong(Expected, Exchange.Ptrs.load(std::memory_order_relaxed), std::memory_order_acq_rel, std::memory_order_relaxed);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const FIndexedPointer& Other) const
	{
		return Ptrs.load(std::memory_order_relaxed) == Other.Ptrs.load(std::memory_order_relaxed);
	}
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const FIndexedPointer& Other) const
	{
		return Ptrs.load(std::memory_order_relaxed) != Other.Ptrs.load(std::memory_order_relaxed);
	}

private:
	std::atomic<uint64> Ptrs;

} GCC_ALIGN(8);

struct FIndexedLockFreeLink
{
	FIndexedPointer     DoubleNext;
	std::atomic<void*>  Payload;
	std::atomic<uint32> SingleNext;
};

// there is a version of this code that uses 128 bit atomics to avoid the indirection, that is why we have this policy class at all.
struct FLockFreeLinkPolicy
{
	enum
	{
		MAX_BITS_IN_TLinkPtr = MAX_LOCK_FREE_LINKS_AS_BITS
	};
	typedef FIndexedPointer TDoublePtr;
	typedef FIndexedLockFreeLink TLink;
	typedef uint32 TLinkPtr;
	typedef TLockFreeAllocOnceIndexedAllocator<FIndexedLockFreeLink, MAX_LOCK_FREE_LINKS, 16384> TAllocator;

	[[nodiscard]] static UE_FORCEINLINE_HINT FIndexedLockFreeLink* DerefLink(uint32 Ptr)
	{
		return LinkAllocator.GetItem(Ptr);
	}
	[[nodiscard]] static UE_FORCEINLINE_HINT FIndexedLockFreeLink* IndexToLink(uint32 Index)
	{
		return LinkAllocator.GetItem(Index);
	}
	[[nodiscard]] static UE_FORCEINLINE_HINT uint32 IndexToPtr(uint32 Index)
	{
		return Index;
	}

	CORE_API static uint32 AllocLockFreeLink();
	CORE_API static void FreeLockFreeLink(uint32 Item);
	CORE_API static TAllocator LinkAllocator;
};

template<int TPaddingForCacheContention, uint64 TABAInc = 1>
class FLockFreePointerListLIFORoot
{
	UE_NONCOPYABLE(FLockFreePointerListLIFORoot)

	typedef FLockFreeLinkPolicy::TDoublePtr TDoublePtr;
	typedef FLockFreeLinkPolicy::TLink TLink;
	typedef FLockFreeLinkPolicy::TLinkPtr TLinkPtr;

public:
	[[nodiscard]] inline FLockFreePointerListLIFORoot()
	{
		// We want to make sure we have quite a lot of extra counter values to avoid the ABA problem. This could probably be relaxed, but eventually it will be dangerous. 
		// The question is "how many queue operations can a thread starve for".
		static_assert(MAX_TagBitsValue / TABAInc >= (1 << 23), "risk of ABA problem");
		static_assert((TABAInc & (TABAInc - 1)) == 0, "must be power of two");
		Reset();
	}

	void Reset()
	{
		Head.Init();
	}

	void Push(TLinkPtr Item)
	{
		while (true)
		{
			TDoublePtr LocalHead;
			LocalHead.AtomicRead(Head);
			TDoublePtr NewHead;
			NewHead.AdvanceCounterAndState(LocalHead, TABAInc);
			NewHead.SetPtr(Item);
			FLockFreeLinkPolicy::DerefLink(Item)->SingleNext.store(LocalHead.GetPtr(), std::memory_order_relaxed);
			if (Head.InterlockedCompareExchange(NewHead, LocalHead))
			{
				break;
			}
		}
	}

	bool PushIf(TFunctionRef<TLinkPtr(uint64)> AllocateIfOkToPush)
	{
		static_assert(TABAInc > 1, "method should not be used for lists without state");
		while (true)
		{
			TDoublePtr LocalHead;
			LocalHead.AtomicRead(Head);
			uint64 LocalState = LocalHead.GetState<TABAInc>();
			TLinkPtr Item = AllocateIfOkToPush(LocalState);
			if (!Item)
			{
				return false;
			}

			TDoublePtr NewHead;
			NewHead.AdvanceCounterAndState(LocalHead, TABAInc);
			FLockFreeLinkPolicy::DerefLink(Item)->SingleNext.store(LocalHead.GetPtr(), std::memory_order_relaxed);
			NewHead.SetPtr(Item);
			if (Head.InterlockedCompareExchange(NewHead, LocalHead))
			{
				break;
			}
		}
		return true;
	}


	TLinkPtr Pop()
	{
		TLinkPtr Item = 0;
		while (true)
		{
			TDoublePtr LocalHead;
			LocalHead.AtomicRead(Head);
			Item = LocalHead.GetPtr();
			if (!Item)
			{
				break;
			}
			TDoublePtr NewHead;
			NewHead.AdvanceCounterAndState(LocalHead, TABAInc);
			TLink* ItemP = FLockFreeLinkPolicy::DerefLink(Item);
			NewHead.SetPtr(ItemP->SingleNext.load(std::memory_order_relaxed));
			if (Head.InterlockedCompareExchange(NewHead, LocalHead))
			{
				ItemP->SingleNext.store(0, std::memory_order_relaxed);
				break;
			}
		}
		return Item;
	}

	TLinkPtr PopAll()
	{
		TLinkPtr Item = 0;
		while (true)
		{
			TDoublePtr LocalHead;
			LocalHead.AtomicRead(Head);
			Item = LocalHead.GetPtr();
			if (!Item)
			{
				break;
			}
			TDoublePtr NewHead;
			NewHead.AdvanceCounterAndState(LocalHead, TABAInc);
			NewHead.SetPtr(0);
			if (Head.InterlockedCompareExchange(NewHead, LocalHead))
			{
				break;
			}
		}
		return Item;
	}

	TLinkPtr PopAllAndChangeState(TFunctionRef<uint64(uint64)> StateChange)
	{
		static_assert(TABAInc > 1, "method should not be used for lists without state");
		TLinkPtr Item = 0;
		while (true)
		{
			TDoublePtr LocalHead;
			LocalHead.AtomicRead(Head);
			Item = LocalHead.GetPtr();
			TDoublePtr NewHead;
			NewHead.AdvanceCounterAndState(LocalHead, TABAInc);
			NewHead.SetState<TABAInc>(StateChange(LocalHead.GetState<TABAInc>()));
			NewHead.SetPtr(0);
			if (Head.InterlockedCompareExchange(NewHead, LocalHead))
			{
				break;
			}
		}
		return Item;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return !Head.GetPtr();
	}

	[[nodiscard]] inline uint64 GetState() const
	{
		TDoublePtr LocalHead;
		LocalHead.AtomicRead(Head);
		return LocalHead.GetState<TABAInc>();
	}

private:
	alignas(TPaddingForCacheContention) TDoublePtr Head;
};

template<class T, int TPaddingForCacheContention, uint64 TABAInc = 1>
class FLockFreePointerListLIFOBase
{
	UE_NONCOPYABLE(FLockFreePointerListLIFOBase)

	typedef FLockFreeLinkPolicy::TDoublePtr TDoublePtr;
	typedef FLockFreeLinkPolicy::TLink TLink;
	typedef FLockFreeLinkPolicy::TLinkPtr TLinkPtr;

public:
	[[nodiscard]] FLockFreePointerListLIFOBase() = default;

	~FLockFreePointerListLIFOBase()
	{
		while (Pop()) {};
	}

	void Reset()
	{
		while (Pop()) {};
		RootList.Reset();
	}

	void Push(T* InPayload)
	{
		TLinkPtr Item = FLockFreeLinkPolicy::AllocLockFreeLink();
		FLockFreeLinkPolicy::DerefLink(Item)->Payload.store(InPayload, std::memory_order_relaxed);
		RootList.Push(Item);
	}

	bool PushIf(T* InPayload, TFunctionRef<bool(uint64)> OkToPush)
	{
		TLinkPtr Item = 0;

		auto AllocateIfOkToPush = [&OkToPush, InPayload, &Item](uint64 State)->TLinkPtr
		{
			if (OkToPush(State))
			{
				if (!Item)
				{
					Item = FLockFreeLinkPolicy::AllocLockFreeLink();
					FLockFreeLinkPolicy::DerefLink(Item)->Payload.store(InPayload, std::memory_order_relaxed);
				}
				return Item;
			}
			return 0;
		};
		if (!RootList.PushIf(AllocateIfOkToPush))
		{
			if (Item)
			{
				// we allocated the link, but it turned out that the list was closed
				FLockFreeLinkPolicy::FreeLockFreeLink(Item);
			}
			return false;
		}
		return true;
	}


	[[nodiscard]] T* Pop()
	{
		TLinkPtr Item = RootList.Pop();
		T* Result = nullptr;
		if (Item)
		{
			Result = (T*)FLockFreeLinkPolicy::DerefLink(Item)->Payload.load(std::memory_order_relaxed);
			FLockFreeLinkPolicy::FreeLockFreeLink(Item);
		}
		return Result;
	}

	template <typename ContainerType>
	void PopAll(ContainerType& OutContainer)
	{
		TLinkPtr Links = RootList.PopAll();
		while (Links)
		{
			TLink* LinksP = FLockFreeLinkPolicy::DerefLink(Links);
			OutContainer.Add((T*)LinksP->Payload.load(std::memory_order_relaxed));
			TLinkPtr Del = Links;
			Links = LinksP->SingleNext.load(std::memory_order_relaxed);
			FLockFreeLinkPolicy::FreeLockFreeLink(Del);
		}
	}

	template <typename FunctorType>
	void PopAllAndApply(FunctorType InFunctor)
	{
		TLinkPtr Links = RootList.PopAll();
		while (Links)
		{
			TLink* LinksP = FLockFreeLinkPolicy::DerefLink(Links);
			InFunctor((T*)LinksP->Payload.load(std::memory_order_relaxed));
			TLinkPtr Del = Links;
			Links = LinksP->SingleNext.load(std::memory_order_relaxed);
			FLockFreeLinkPolicy::FreeLockFreeLink(Del);
		}
	}

	template <typename ContainerType>
	void PopAllAndChangeState(ContainerType& OutContainer, TFunctionRef<uint64(uint64)> StateChange)
	{
		TLinkPtr Links = RootList.PopAllAndChangeState(StateChange);
		while (Links)
		{
			TLink* LinksP = FLockFreeLinkPolicy::DerefLink(Links);
			OutContainer.Add((T*)LinksP->Payload.load(std::memory_order_relaxed));
			TLinkPtr Del = Links;
			Links = LinksP->SingleNext.load(std::memory_order_relaxed);
			FLockFreeLinkPolicy::FreeLockFreeLink(Del);
		}
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return RootList.IsEmpty();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT uint64 GetState() const
	{
		return RootList.GetState();
	}

private:

	FLockFreePointerListLIFORoot<TPaddingForCacheContention, TABAInc> RootList;
};

template<class T, int TPaddingForCacheContention, uint64 TABAInc = 1>
class FLockFreePointerFIFOBase
{
	UE_NONCOPYABLE(FLockFreePointerFIFOBase)

	typedef FLockFreeLinkPolicy::TDoublePtr TDoublePtr;
	typedef FLockFreeLinkPolicy::TLink TLink;
	typedef FLockFreeLinkPolicy::TLinkPtr TLinkPtr;
public:

	[[nodiscard]] inline FLockFreePointerFIFOBase()
	{
		// We want to make sure we have quite a lot of extra counter values to avoid the ABA problem. This could probably be relaxed, but eventually it will be dangerous. 
		// The question is "how many queue operations can a thread starve for".
		static_assert(TABAInc <= 65536, "risk of ABA problem");
		static_assert((TABAInc & (TABAInc - 1)) == 0, "must be power of two");

		Head.Init();
		Tail.Init();
		TLinkPtr Stub = FLockFreeLinkPolicy::AllocLockFreeLink();
		Head.SetPtr(Stub);
		Tail.SetPtr(Stub);
	}

	~FLockFreePointerFIFOBase()
	{
		while (Pop()) {};
		FLockFreeLinkPolicy::FreeLockFreeLink(Head.GetPtr());
	}

	void Push(T* InPayload)
	{
		TLinkPtr Item = FLockFreeLinkPolicy::AllocLockFreeLink();
		FLockFreeLinkPolicy::DerefLink(Item)->Payload.store(InPayload, std::memory_order_relaxed);
		TDoublePtr LocalTail;
		while (true)
		{
			LocalTail.AtomicRead(Tail);
			TLink* LocalTailP = FLockFreeLinkPolicy::DerefLink(LocalTail.GetPtr());
			TDoublePtr LocalNext;
			LocalNext.AtomicRead(LocalTailP->DoubleNext);
			TDoublePtr TestLocalTail;
			TestLocalTail.AtomicRead(Tail);
			if (TestLocalTail == LocalTail)
			{
				if (LocalNext.GetPtr())
				{
					TestCriticalStall();
					TDoublePtr NewTail;
					NewTail.AdvanceCounterAndState(LocalTail, TABAInc);
					NewTail.SetPtr(LocalNext.GetPtr());
					Tail.InterlockedCompareExchange(NewTail, LocalTail);
				}
				else
				{
					TestCriticalStall();
					TDoublePtr NewNext;
					NewNext.AdvanceCounterAndState(LocalNext, TABAInc);
					NewNext.SetPtr(Item);
					if (LocalTailP->DoubleNext.InterlockedCompareExchange(NewNext, LocalNext))
					{
						break;
					}
				}
			}
		}
		{
			TestCriticalStall();
			TDoublePtr NewTail;
			NewTail.AdvanceCounterAndState(LocalTail, TABAInc);
			NewTail.SetPtr(Item);
			Tail.InterlockedCompareExchange(NewTail, LocalTail);
		}
	}

	[[nodiscard]] T* Pop()
	{
		T* Result = nullptr;
		TDoublePtr LocalHead;
		while (true)
		{
			LocalHead.AtomicRead(Head);
			TDoublePtr LocalTail;
			LocalTail.AtomicRead(Tail);
			TDoublePtr LocalNext;
			LocalNext.AtomicRead(FLockFreeLinkPolicy::DerefLink(LocalHead.GetPtr())->DoubleNext);
			TDoublePtr LocalHeadTest;
			LocalHeadTest.AtomicRead(Head);
			if (LocalHead == LocalHeadTest)
			{
				if (LocalHead.GetPtr() == LocalTail.GetPtr())
				{
					if (!LocalNext.GetPtr())
					{
						return nullptr;
					}
					TestCriticalStall();
					TDoublePtr NewTail;
					NewTail.AdvanceCounterAndState(LocalTail, TABAInc);
					NewTail.SetPtr(LocalNext.GetPtr());
					Tail.InterlockedCompareExchange(NewTail, LocalTail);
				}
				else
				{
					TestCriticalStall();
					Result = (T*)FLockFreeLinkPolicy::DerefLink(LocalNext.GetPtr())->Payload.load(std::memory_order_relaxed);
					TDoublePtr NewHead;
					NewHead.AdvanceCounterAndState(LocalHead, TABAInc);
					NewHead.SetPtr(LocalNext.GetPtr());
					if (Head.InterlockedCompareExchange(NewHead, LocalHead))
					{
						break;
					}
				}
			}
		}
		FLockFreeLinkPolicy::FreeLockFreeLink(LocalHead.GetPtr());
		return Result;
	}

	template <typename ContainerType>
	void PopAll(ContainerType& OutContainer)
	{
		while (T* Item = Pop())
		{
			OutContainer.Add(Item);
		}
	}


	[[nodiscard]] inline bool IsEmpty() const
	{
		TDoublePtr LocalHead;
		LocalHead.AtomicRead(Head);
		TDoublePtr LocalNext;
		LocalNext.AtomicRead(FLockFreeLinkPolicy::DerefLink(LocalHead.GetPtr())->DoubleNext);
		return !LocalNext.GetPtr();
	}

private:
	alignas(TPaddingForCacheContention) TDoublePtr Head;
	alignas(TPaddingForCacheContention) TDoublePtr Tail;
};


template<class T, int TPaddingForCacheContention, int NumPriorities>
class FStallingTaskQueue
{
	UE_NONCOPYABLE(FStallingTaskQueue)

	typedef FLockFreeLinkPolicy::TDoublePtr TDoublePtr;
	typedef FLockFreeLinkPolicy::TLink TLink;
	typedef FLockFreeLinkPolicy::TLinkPtr TLinkPtr;
public:
	[[nodiscard]] FStallingTaskQueue()
	{
		MasterState.Init();
	}
	int32 Push(T* InPayload, uint32 Priority)
	{
		checkLockFreePointerList(Priority < NumPriorities);
		TDoublePtr LocalMasterState;
		LocalMasterState.AtomicRead(MasterState);
		PriorityQueues[Priority].Push(InPayload);
		TDoublePtr NewMasterState;
		NewMasterState.AdvanceCounterAndState(LocalMasterState, 1);
		int32 ThreadToWake = FindThreadToWake(LocalMasterState.GetPtr());
		if (ThreadToWake >= 0)
		{
			NewMasterState.SetPtr(TurnOffBit(LocalMasterState.GetPtr(), ThreadToWake));
		}
		else
		{
			NewMasterState.SetPtr(LocalMasterState.GetPtr());
		}
		while (!MasterState.InterlockedCompareExchange(NewMasterState, LocalMasterState))
		{
			LocalMasterState.AtomicRead(MasterState);
			NewMasterState.AdvanceCounterAndState(LocalMasterState, 1);
			ThreadToWake = FindThreadToWake(LocalMasterState.GetPtr());
#if 0
			// This block of code is supposed to avoid starting the task thread if the queues are empty.
			// There WAS a silly bug here. In rare cases no task thread is woken up.
			// That bug has been fixed, but I don't think we really need this code anyway.
			// Without this block, it is possible that we do a redundant wake-up, but for task threads, that can happen anyway. 
			// For named threads, the rare redundant wakeup seems acceptable.
			if (ThreadToWake >= 0)
			{
				bool bAny = false;
				for (int32 Index = 0; !bAny && Index < NumPriorities; Index++)
				{
					bAny = !PriorityQueues[Index].IsEmpty();
				}
				if (!bAny) // if there is nothing in the queues, then don't wake anyone
				{
					ThreadToWake = -1;
				}
			}
#endif
			if (ThreadToWake >= 0)
			{
				NewMasterState.SetPtr(TurnOffBit(LocalMasterState.GetPtr(), ThreadToWake));
			}
			else
			{
				NewMasterState.SetPtr(LocalMasterState.GetPtr());
			}
		}
		return ThreadToWake;
	}

	[[nodiscard]] T* Pop(int32 MyThread, bool bAllowStall)
	{
		check(MyThread >= 0 && MyThread < FLockFreeLinkPolicy::MAX_BITS_IN_TLinkPtr);

		while (true)
		{
			TDoublePtr LocalMasterState;
			LocalMasterState.AtomicRead(MasterState);
			//checkLockFreePointerList(!TestBit(LocalMasterState.GetPtr(), MyThread) || !FPlatformProcess::SupportsMultithreading()); // you should not be stalled if you are asking for a task
			for (int32 Index = 0; Index < NumPriorities; Index++)
			{
				T *Result = PriorityQueues[Index].Pop();
				if (Result)
				{
					while (true)
					{
						TDoublePtr NewMasterState;
						NewMasterState.AdvanceCounterAndState(LocalMasterState, 1);
						NewMasterState.SetPtr(LocalMasterState.GetPtr());
						if (MasterState.InterlockedCompareExchange(NewMasterState, LocalMasterState))
						{
							return Result;
						}
						LocalMasterState.AtomicRead(MasterState);
						checkLockFreePointerList(!TestBit(LocalMasterState.GetPtr(), MyThread) || !FPlatformProcess::SupportsMultithreading()); // you should not be stalled if you are asking for a task
					}
				}
			}
			if (!bAllowStall)
			{
				break; // if we aren't stalling, we are done, the queues are empty
			}
			{
				TDoublePtr NewMasterState;
				NewMasterState.AdvanceCounterAndState(LocalMasterState, 1);
				NewMasterState.SetPtr(TurnOnBit(LocalMasterState.GetPtr(), MyThread));
				if (MasterState.InterlockedCompareExchange(NewMasterState, LocalMasterState))
				{
					break;
				}
			}
		}
		return nullptr;
	}

private:

	[[nodiscard]] static int32 FindThreadToWake(TLinkPtr Ptr)
	{
		int32 Result = -1;
		UPTRINT Test = UPTRINT(Ptr);
		if (Test)
		{
			Result = 0;
			while (!(Test & 1))
			{
				Test >>= 1;
				Result++;
			}
		}
		return Result;
	}

	[[nodiscard]] static TLinkPtr TurnOffBit(TLinkPtr Ptr, int32 BitToTurnOff)
	{
		return (TLinkPtr)(UPTRINT(Ptr) & ~(UPTRINT(1) << BitToTurnOff));
	}

	[[nodiscard]] static TLinkPtr TurnOnBit(TLinkPtr Ptr, int32 BitToTurnOn)
	{
		return (TLinkPtr)(UPTRINT(Ptr) | (UPTRINT(1) << BitToTurnOn));
	}

	[[nodiscard]] static bool TestBit(TLinkPtr Ptr, int32 BitToTest)
	{
		return !!(UPTRINT(Ptr) & (UPTRINT(1) << BitToTest));
	}

	FLockFreePointerFIFOBase<T, TPaddingForCacheContention> PriorityQueues[NumPriorities];
	// not a pointer to anything, rather tracks the stall state of all threads servicing this queue.
	alignas(TPaddingForCacheContention) TDoublePtr MasterState;
};




template<class T, int TPaddingForCacheContention>
class TLockFreePointerListLIFOPad : private FLockFreePointerListLIFOBase<T, TPaddingForCacheContention>
{
public:

	/**
	*	Push an item onto the head of the list.
	*
	*	@param NewItem, the new item to push on the list, cannot be NULL.
	*/
	void Push(T *NewItem)
	{
		FLockFreePointerListLIFOBase<T, TPaddingForCacheContention>::Push(NewItem);
	}

	/**
	*	Pop an item from the list or return NULL if the list is empty.
	*	@return The popped item, if any.
	*/
	[[nodiscard]] T* Pop()
	{
		return FLockFreePointerListLIFOBase<T, TPaddingForCacheContention>::Pop();
	}

	/**
	*	Pop all items from the list.
	*
	*	@param Output The array to hold the returned items. Must be empty.
	*/
	template <typename ContainerType>
	void PopAll(ContainerType& Output)
	{
		FLockFreePointerListLIFOBase<T, TPaddingForCacheContention>::PopAll(Output);
	}

	/**
	*	Pop all items from the list and call a functor for each of them.
	*/
	template <typename FunctorType>
	void PopAllAndApply(FunctorType InFunctor)
	{
		FLockFreePointerListLIFOBase<T, TPaddingForCacheContention>::PopAllAndApply(InFunctor);
	}

	/**
	*	Check if the list is empty.
	*
	*	@return true if the list is empty.
	*	CAUTION: This methods safety depends on external assumptions. For example, if another thread could add to the list at any time, the return value is no better than a best guess.
	*	As typically used, the list is not being access concurrently when this is called.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return FLockFreePointerListLIFOBase<T, TPaddingForCacheContention>::IsEmpty();
	}
};

template<class T>
class TLockFreePointerListLIFO : public TLockFreePointerListLIFOPad<T, 0>
{

};

template<class T, int TPaddingForCacheContention>
class TLockFreePointerListUnordered : public TLockFreePointerListLIFOPad<T, TPaddingForCacheContention>
{

};

template<class T, int TPaddingForCacheContention>
class TLockFreePointerListFIFO : private FLockFreePointerFIFOBase<T, TPaddingForCacheContention>
{
public:

	/**
	*	Push an item onto the head of the list.
	*
	*	@param NewItem, the new item to push on the list, cannot be NULL.
	*/
	void Push(T *NewItem)
	{
		FLockFreePointerFIFOBase<T, TPaddingForCacheContention>::Push(NewItem);
	}

	/**
	*	Pop an item from the list or return NULL if the list is empty.
	*	@return The popped item, if any.
	*/
	[[nodiscard]] T* Pop()
	{
		return FLockFreePointerFIFOBase<T, TPaddingForCacheContention>::Pop();
	}

	/**
	*	Pop all items from the list.
	*
	*	@param Output The array to hold the returned items. Must be empty.
	*/
	template <typename ContainerType>
	void PopAll(ContainerType& Output)
	{
		FLockFreePointerFIFOBase<T, TPaddingForCacheContention>::PopAll(Output);
	}

	/**
	*	Check if the list is empty.
	*
	*	@return true if the list is empty.
	*	CAUTION: This methods safety depends on external assumptions. For example, if another thread could add to the list at any time, the return value is no better than a best guess.
	*	As typically used, the list is not being access concurrently when this is called.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return FLockFreePointerFIFOBase<T, TPaddingForCacheContention>::IsEmpty();
	}
};


template<class T, int TPaddingForCacheContention>
class TClosableLockFreePointerListUnorderedSingleConsumer : private FLockFreePointerListLIFOBase<T, TPaddingForCacheContention, 2>
{
public:

	/**
	*	Reset the list to the initial state. Not thread safe, but used for recycling when we know all users are gone.
	*/
	void Reset()
	{
		FLockFreePointerListLIFOBase<T, TPaddingForCacheContention, 2>::Reset();
	}

	/**
	*	Push an item onto the head of the list, unless the list is closed
	*
	*	@param NewItem, the new item to push on the list, cannot be NULL
	*	@return true if the item was pushed on the list, false if the list was closed.
	*/
	bool PushIfNotClosed(T *NewItem)
	{
		return FLockFreePointerListLIFOBase<T, TPaddingForCacheContention, 2>::PushIf(NewItem, [](uint64 State)->bool {return !(State & 1); });
	}

	/**
	*	Pop all items from the list and atomically close it.
	*
	*	@param Output The array to hold the returned items. Must be empty.
	*/
	template <typename ContainerType>
	void PopAllAndClose(ContainerType& Output)
	{
		auto CheckOpenAndClose = [](uint64 State) -> uint64
		{
			checkLockFreePointerList(!(State & 1));
			return State | 1;
		};
		FLockFreePointerListLIFOBase<T, TPaddingForCacheContention, 2>::PopAllAndChangeState(Output, CheckOpenAndClose);
	}

	/**
	*	Check if the list is closed
	*
	*	@return true if the list is closed.
	*/
	[[nodiscard]] bool IsClosed() const
	{
		return !!(FLockFreePointerListLIFOBase<T, TPaddingForCacheContention, 2>::GetState() & 1);
	}

};


