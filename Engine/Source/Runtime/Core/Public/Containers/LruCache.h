// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/AllowShrinking.h"
#include "Containers/Set.h"
#include "Misc/AssertionMacros.h"


/**
 * Default comparer for keys in TLruCache.
 *
 * @param KeyType The type of keys to compare.
 */
template<typename KeyType>
struct DefaultKeyComparer
{
	[[nodiscard]] static UE_FORCEINLINE_HINT bool Matches(KeyType A, KeyType B)
	{
		return (A == B);
	}

	/** Calculates a hash index for a key. */
	[[nodiscard]] static UE_FORCEINLINE_HINT uint32 GetKeyHash(KeyType Key)
	{
		return GetTypeHash(Key);
	}
};


/**
 * Implements a Least Recently Used (LRU) cache.
 *
 * @param KeyType The type of cache entry keys.
 * @param ValueType The type of cache entry values.
 * @param KeyFuncs Optional functions for comparing keys in the lookup set (see BaseKeyFuncs in Set.h).
 */
template<typename KeyType, typename ValueType, typename KeyComp = DefaultKeyComparer<KeyType> >
class TLruCache
{
	/** An entry in the LRU cache. */
	struct FCacheEntry
	{
		/** The entry's lookup key. */
		KeyType Key;

		/** The less recent entry in the linked list. */
		FCacheEntry* LessRecent;

		/** The more recent entry in the linked list. */
		FCacheEntry* MoreRecent;

		/** The entry's value. */
		ValueType Value;

		/**
		 * Create and initialize a new instance.
		 *
		 * @param InKey The entry's key.
		 * @param InValue The entry's value.
		 */
		[[nodiscard]] FCacheEntry(const KeyType& InKey, const ValueType& InValue)
			: Key(InKey)
			, LessRecent(nullptr)
			, MoreRecent(nullptr)
			, Value(InValue)
		{
		}

		/**
		 * Create a new instance with a default key value.
		 *
		 * @param InKey The entry's key.
		 */
		[[nodiscard]] FCacheEntry(const KeyType& InKey)
			: Key(InKey)
			, LessRecent(nullptr)
			, MoreRecent(nullptr)
		{
		}


		/** Add this entry before the given one. */
		inline void LinkBefore(FCacheEntry* Other)
		{
			LessRecent = Other;

			if (Other != nullptr)
			{
				Other->MoreRecent = this;
			}
		}

		/** Remove this entry from the list. */
		inline void Unlink()
		{
			if (LessRecent != nullptr)
			{
				LessRecent->MoreRecent = MoreRecent;
			}

			if (MoreRecent != nullptr)
			{
				MoreRecent->LessRecent = LessRecent;
			}

			LessRecent = nullptr;
			MoreRecent = nullptr;
		}
	};

	/** Lookup set key functions. */
	struct FKeyFuncs : public BaseKeyFuncs<FCacheEntry*, KeyType>
	{
		[[nodiscard]] UE_FORCEINLINE_HINT static const KeyType& GetSetKey(const FCacheEntry* Entry)
		{
			return Entry->Key;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT static bool Matches(KeyType A, KeyType B)
		{
			return KeyComp::Matches(A, B);
		}

		[[nodiscard]] UE_FORCEINLINE_HINT static uint32 GetKeyHash(KeyType Key)
		{
			return KeyComp::GetKeyHash(Key);
		}
	};

public:

	/** Default constructor (empty cache that cannot hold any values). */
	[[nodiscard]] TLruCache()
		: LeastRecent(nullptr)
		, MostRecent(nullptr)
		, MaxNumElements(0)
	{
	}

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InMaxNumElements The maximum number of elements this cache can hold.
	 */
	[[nodiscard]] TLruCache(int32 InMaxNumElements)
		: LeastRecent(nullptr)
		, MostRecent(nullptr)
		, MaxNumElements(InMaxNumElements)
	{
		Empty(InMaxNumElements);
	}

	TLruCache(const TLruCache& Other)
	{
		*this = Other;
	}

	TLruCache& operator=(const TLruCache& Other)
	{
		Empty(Other.MaxNumElements);

		// Copy from least recent to most so that we keep the usage state through copy
		const FCacheEntry* CurrentEntry = Other.LeastRecent;
		while (CurrentEntry)
		{
			Add(CurrentEntry->Key, CurrentEntry->Value);
			CurrentEntry = CurrentEntry->MoreRecent;
		}

		return *this;
	}

	TLruCache(TLruCache&& Other) = default;
	TLruCache& operator=(TLruCache&& Other) = default;

	/** Destructor. */
	~TLruCache()
	{
		Empty();
	}

public:

	/**
	 * Add an entry to the cache.
	 *
	 * If an entry with the specified key already exists in the cache,
	 * the value of the existing entry will be updated. The added or
	 * updated entry will be marked as the most recently used one.
	 *
	 * @param Key The entry's lookup key.
	 * @param Value The entry's value.
	 * @see AddUninitialized_GetRef,Empty, Find, GetKeys, Remove
	 */
	void Add(const KeyType& Key, const ValueType& Value)
	{
		check(MaxNumElements != 0 && "Cannot add values to zero size TLruCache");

		FCacheEntry** EntryPtr = LookupSet.Find(Key);

		if (EntryPtr != nullptr)
		{
			// update existing entry
			FCacheEntry* Entry = *EntryPtr;
			checkSlow(Entry->Key == Key);

			Entry->Value = Value;
			MarkAsRecent(*Entry);
		}
		else
		{
			// add new entry
			if (LookupSet.Num() == MaxNumElements)
			{
				Remove(LeastRecent);
			}

			FCacheEntry* NewEntry = new FCacheEntry(Key, Value);
			NewEntry->LinkBefore(MostRecent);
			MostRecent = NewEntry;

			if (LeastRecent == nullptr)
			{
				LeastRecent = NewEntry;
			}

			LookupSet.Add(NewEntry);
		}
	}

	/**
	 * Add an entry to the cache.
	 *
	 * If an entry with the specified key already exists in the cache,
	 * the value of the existing entry will be returned. The added or
	 * updated entry will be marked as the most recently used one.
	 *
	 * @param Key The entry's lookup key.
	 * @return The entry's value.
	 * @see Add, Empty, Find, GetKeys, Remove
	 */
	ValueType& AddUninitialized_GetRef(const KeyType& Key)
	{
		check(MaxNumElements != 0 && "Cannot add values to zero size TLruCache");

		FCacheEntry** EntryPtr = LookupSet.Find(Key);

		if (EntryPtr != nullptr)
		{
			// update existing entry
			FCacheEntry* Entry = *EntryPtr;
			checkSlow(Entry->Key == Key);

			MarkAsRecent(*Entry);
			return Entry->Value;
		}
		else
		{
			// add new entry
			if (LookupSet.Num() == MaxNumElements)
			{
				Remove(LeastRecent);
			}

			FCacheEntry* NewEntry = new FCacheEntry(Key);
			NewEntry->LinkBefore(MostRecent);
			MostRecent = NewEntry;

			if (LeastRecent == nullptr)
			{
				LeastRecent = NewEntry;
			}

			LookupSet.Add(NewEntry);
			return NewEntry->Value;
		}
	}



	/**
	 * Check whether an entry with the specified key is in the cache.
	 *
	 * @param Key The key of the entry to check.
	 * @return true if the entry is in the cache, false otherwise.
	 * @see Add, ContainsByPredicate, Empty, FilterByPredicate, Find, GetKeys, Remove
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool Contains(const KeyType& Key) const
	{
		return LookupSet.Contains(Key);
	}

	/**
	 * Check whether an entry for which a predicate returns true is in the cache.
	 *
	 * @param Pred The predicate functor to apply to each entry.
	 * @return true if at least one matching entry is in the cache, false otherwise.
	 * @see Contains, FilterByPredicate, FindByPredicate, RemoveByPredicate
	 */
	template<typename Predicate>
	[[nodiscard]] inline bool ContainsByPredicate(Predicate Pred) const
	{
		for (const FCacheEntry* Entry : LookupSet)
		{
			if (Pred(Entry->Key, Entry->Value))
			{
				return true;
			}
		}

		return false;
	}

	/**
	 * Empty the cache.
	 *
	 * @param InMaxNumElements The maximum number of elements this cache can hold (default = 0).
	 * @see Add, Find, GetKeys, Max, Num, Remove
	 */
	void Empty(int32 InMaxNumElements = 0)
	{
		for (FCacheEntry* Entry : LookupSet)
		{
			delete Entry;
		}

		MaxNumElements = InMaxNumElements;
		LookupSet.Empty(FMath::Max(MaxNumElements, 0));

		MostRecent = nullptr;
		LeastRecent = nullptr;
	}

	/**
	 * Change max cache size.
	 * 
	 * @param InMaxNumElements The maximum number of elements this cache can hold
	 * @param InAllowShrinking If InMaxNumElements is smaller than current MaxNumElements, indicates if we want to compact internal memory
	 */
	void SetMaxNumElements(int32 InMaxNumElements, EAllowShrinking InAllowShrinking = EAllowShrinking::Default)
	{
		if (InMaxNumElements < MaxNumElements)
		{
			while (LookupSet.Num() > InMaxNumElements)
			{
				RemoveLeastRecent();
			}

			if (InAllowShrinking == EAllowShrinking::Yes)
			{
				LookupSet.Shrink();
			}
		}

		LookupSet.Reserve(InMaxNumElements);
		MaxNumElements = InMaxNumElements;
	}

	/**
	 * Filter the entries in the cache using a predicate.
	 *
	 * @param Pred The predicate functor to apply to each entry.
	 * @return Collection of values for which the predicate returned true.
	 * @see ContainsByPredicate, FindByPredicate, Find, RemoveByPredicate
	 */
	template<typename Predicate>
	[[nodiscard]] TArray<ValueType> FilterByPredicate(Predicate Pred) const
	{
		TArray<ValueType> Result;

		for (const FCacheEntry* Entry : LookupSet)
		{
			if (Pred(Entry->Key, Entry->Value))
			{
				Result.Add(Entry->Value);
			}
		}

		return Result;
	}

	/**
	 * Find the value of the entry with the specified key.
	 *
	 * @param Key The key of the entry to get.
	 * @return Pointer to the value, or nullptr if not found.
	 * @see Add, Contains, Empty, FindAndTouch, GetKeys, Remove
	 */
	[[nodiscard]] inline const ValueType* Find(const KeyType& Key) const
	{
		FCacheEntry*const * EntryPtr = LookupSet.Find(Key);

		if (EntryPtr != nullptr)
		{
			return &(*EntryPtr)->Value;
		}

		return nullptr;
	}

	/**
	 * Find the value of the entry with the specified key.
	 *
	 * @param Key The key of the entry to get.
	 * @return Pointer to the value, or nullptr if not found.
	 * @see Add, Contains, Empty, FindAndTouch, GetKeys, Remove
	 */
	[[nodiscard]] inline ValueType* Find(const KeyType& Key)
	{
		FCacheEntry* const* EntryPtr = LookupSet.Find(Key);

		if (EntryPtr != nullptr)
		{
			return &(*EntryPtr)->Value;
		}

		return nullptr;
	}

	/**
	 * Find the value of the entry with the specified key.
	 *
	 * @param Key The key of the entry to get.
	 * @return Reference to the value, or triggers an assertion if the key does not exist.
	 */
	[[nodiscard]] inline const ValueType& FindChecked(const KeyType& Key) const
	{
		FCacheEntry*const * EntryPtr = LookupSet.Find(Key);

		check(EntryPtr);

		return (*EntryPtr)->Value;
	}

	/**
	 * Find the value of the entry with the specified key.
	 *
	 * @param Key The key of the entry to get.
	 * @return Reference to the value, or triggers an assertion if the key does not exist.
	 */
	[[nodiscard]] inline ValueType& FindChecked(const KeyType& Key)
	{
		FCacheEntry*const * EntryPtr = LookupSet.Find(Key);

		check(EntryPtr);

		return (*EntryPtr)->Value;
	}

	/**
	 * Find the value of the entry with the specified key.
	 *
	 * @param Key The key of the entry to get.
	 * @return Copy of the value, or the default value for the ValueType if the key does not exist.
	 */
	[[nodiscard]] inline ValueType FindRef(const KeyType& Key) const
	{
		FCacheEntry*const * EntryPtr = LookupSet.Find(Key);

		if (EntryPtr != nullptr)
		{
			return (*EntryPtr)->Value;
		}

		return ValueType();
	}

	/**
	 * Find the value of the entry with the specified key and mark it as the most recently used.
	 *
	 * @param Key The key of the entry to get.
	 * @return Pointer to the value, or nullptr if not found.
	 * @see Add, Contains, Empty, Find, GetKeys, Remove
	 */
	[[nodiscard]] ValueType* FindAndTouch(const KeyType& Key)
	{
		FCacheEntry** EntryPtr = LookupSet.Find(Key);

		if (EntryPtr == nullptr)
		{
			return nullptr;
		}

		MarkAsRecent(**EntryPtr);

		return &(*EntryPtr)->Value;
	}

	/**
	 * Find the value of the entry with the specified key and mark it as the most recently used.
	 *
	 * @param Key The key of the entry to get.
	 * @return Pointer to the value, or triggers an assertion if the key does not exist.
	 */
	[[nodiscard]] ValueType& FindAndTouchChecked(const KeyType& Key)
	{
		FCacheEntry** EntryPtr = LookupSet.Find(Key);

		check(EntryPtr);

		MarkAsRecent(**EntryPtr);

		return (*EntryPtr)->Value;
	}

	/**
	 * Find the value of the entry with the specified key and mark it as the most recently used.
	 *
	 * @param Key The key of the entry to get.
	 * @return Copy of the value, or the default value for the ValueType if the key does not exist.
	 */
	[[nodiscard]] ValueType FindAndTouchRef(const KeyType& Key)
	{
		FCacheEntry** EntryPtr = LookupSet.Find(Key);

		if (EntryPtr == nullptr)
		{
			return ValueType();
		}

		MarkAsRecent(**EntryPtr);

		return (*EntryPtr)->Value;
	}

	/**
	 * Find the value of an entry using a predicate.
	 *
	 * @param Pred The predicate functor to apply to each entry.
	 * @return Pointer to value for which the predicate returned true, or nullptr if not found.
	 * @see ContainsByPredicate, FilterByPredicate, RemoveByPredicate
	 */
	template<typename Predicate>
	[[nodiscard]] const ValueType* FindByPredicate(Predicate Pred) const
	{
		for (const FCacheEntry* Entry : LookupSet)
		{
			if (Pred(Entry->Key, Entry->Value))
			{
				return &Entry->Value;
			}
		}

		return nullptr;
	}

	/**
	 * Find the keys of all cached entries.
	 *
	 * @param OutKeys Will contain the collection of keys.
	 * @see Add, Empty, Find
	 */
	void GetKeys(TArray<KeyType>& OutKeys) const
	{
		for (const FCacheEntry* Entry : LookupSet)
		{
			OutKeys.Add(Entry->Key);
		}
	}

	/**
	 * Get the maximum number of entries in the cache.
	 *
	 * @return Maximum number of entries.
	 * @see Empty, Num
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Max() const
	{
		return MaxNumElements;
	}

	/**
	 * Returns true if the cache is empty and contains no elements.
	 *
	 * @returns True if the cache is empty.
	 * @see Num
	 */
	[[nodiscard]] bool IsEmpty() const
	{
		return LookupSet.IsEmpty();
	}

	/**
	 * Get the number of entries in the cache.
	 *
	 * @return Number of entries.
	 * @see Empty, Max
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Num() const
	{
		return LookupSet.Num();
	}

	/**
	 * Remove all entries with the specified key from the cache.
	 *
	 * @param Key The key of the entries to remove.
	 * @see Add, Empty, Find, RemoveByPredicate
	 */
	void Remove(const KeyType& Key)
	{
		FCacheEntry** EntryPtr = LookupSet.Find(Key);

		if (EntryPtr != nullptr)
		{
			Remove(*EntryPtr);
		}
	}

	/**
	 * Remove all entries using a predicate.
	 *
	 * @param Pred The predicate function to apply to each entry.
	 * @return Number of removed entries.
	 * @see ContainsByPredicate, FilterByPredicate, FindByPredicate, Remove
	 */
	template<typename Predicate>
	int32 RemoveByPredicate(Predicate Pred)
	{
		int32 NumRemoved = 0;

		for (auto It = LookupSet.CreateIterator(); It; ++It)
		{
			FCacheEntry* Entry = *It;
			if (Pred(Entry->Key, Entry->Value))
			{
				if (Entry == LeastRecent)
				{
					LeastRecent = Entry->MoreRecent;
				}

				if (Entry == MostRecent)
				{
					MostRecent = Entry->LessRecent;
				}

				It.RemoveCurrent(); // remove before delete so hashkey doesn't change

				Entry->Unlink();
				delete Entry;

				++NumRemoved;
			}
		}

		return NumRemoved;
	}

	/**
	 * Remove and return the least recent element from the cache.
	 *
	 * @return Copy of removed value.
	 */
	inline ValueType RemoveLeastRecent()
	{
		check(LeastRecent);
		ValueType LeastRecentElement = MoveTemp(LeastRecent->Value);
		Remove(LeastRecent);
		return LeastRecentElement;
	}

	/**
	 * Return the least recent element key from the cache.
	 *
	 * @return Copy of least recent key.
	 */
	[[nodiscard]] inline KeyType GetLeastRecentKey() const
	{
		check(LeastRecent);
		return LeastRecent->Key;
	}

public:

	/**
	 * Base class for cache iterators.
	 *
	 * Iteration begins at the most recent entry.
	 */
	template<bool Const>
	class TBaseIterator
	{
	public:

		[[nodiscard]] UE_FORCEINLINE_HINT TBaseIterator()
			: CurrentEntry(nullptr)
		{
		}

		[[nodiscard]] UE_FORCEINLINE_HINT TBaseIterator(const TLruCache& Cache)
			: CurrentEntry(Cache.MostRecent)
		{
		}

	public:

		inline TBaseIterator& operator++()
		{
			Increment();
			return *this;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TBaseIterator& Rhs) const
		{
			return CurrentEntry == Rhs.CurrentEntry;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TBaseIterator& Rhs) const
		{
			return CurrentEntry != Rhs.CurrentEntry;
		}

		[[nodiscard]] ValueType& operator->() const
		{
			check(CurrentEntry != nullptr);
			return CurrentEntry->Value;
		}

		[[nodiscard]] ValueType& operator*() const
		{
			check(CurrentEntry != nullptr);
			return CurrentEntry->Value;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
		{
			return (CurrentEntry != nullptr);
		}

		[[nodiscard]] UE_FORCEINLINE_HINT bool operator!() const
		{
			return !(bool)*this;
		}

	public:

		[[nodiscard]] inline KeyType& Key() const
		{
			check(CurrentEntry != nullptr);
			return CurrentEntry->Key;
		}

		[[nodiscard]] inline ValueType& Value() const
		{
			check(CurrentEntry != nullptr);
			return CurrentEntry->Value;
		}

	protected:

		[[nodiscard]] FCacheEntry* GetCurrentEntry()
		{
			return CurrentEntry;
		}

		void Increment()
		{
			check(CurrentEntry != nullptr);
			CurrentEntry = CurrentEntry->LessRecent;
		}

	private:

		FCacheEntry* CurrentEntry;
	};


	/**
	 * Cache iterator (const).
	 */
	class TConstIterator
		: public TBaseIterator<true>
	{
	public:

		[[nodiscard]] UE_FORCEINLINE_HINT TConstIterator()
			: TBaseIterator<true>()
		{
		}

		[[nodiscard]] UE_FORCEINLINE_HINT TConstIterator(const TLruCache& Cache)
			: TBaseIterator<true>(Cache)
		{
		}
	};


	/**
	 * Cache iterator.
	 */
	class TIterator
		: public TBaseIterator<false>
	{
	public:

		[[nodiscard]] inline TIterator()
			: TBaseIterator<false>()
			, Cache(nullptr)
		{
		}

		[[nodiscard]] inline TIterator(TLruCache& InCache)
			: TBaseIterator<false>(InCache)
			, Cache(&InCache)
		{
		}

		/** Removes the current element from the cache and increments the iterator. */
		inline void RemoveCurrentAndIncrement()
		{
			check(Cache != nullptr);

			FCacheEntry* MoreRecentEntry = this->GetCurrentEntry();
			this->Increment();
			Cache->Remove(MoreRecentEntry);
		}

	private:

		TLruCache* Cache;
	};

protected:

	/**
	 * Mark the given entry as recently used.
	 *
	 * @param Entry The entry to mark.
	 */
	inline void MarkAsRecent(FCacheEntry& Entry)
	{
		check(LeastRecent != nullptr);
		check(MostRecent != nullptr);

		// if entry is least recent and not the only item in the list, make it not least recent
		if ((&Entry == LeastRecent) && (LeastRecent->MoreRecent != nullptr))
		{
			LeastRecent = LeastRecent->MoreRecent;
		}

		// relink if not already the most recent item
		if (&Entry != MostRecent)
		{
			Entry.Unlink();
			Entry.LinkBefore(MostRecent);
			MostRecent = &Entry;
		}
	}

	/**
	 * Remove the specified entry from the cache.
	 *
	 * @param Entry The entry to remove.
	 */
	inline void Remove(FCacheEntry* Entry)
	{
		if (Entry == nullptr)
		{
			return;
		}

		LookupSet.Remove(Entry->Key);

		if (Entry == LeastRecent)
		{
			LeastRecent = Entry->MoreRecent;
		}

		if (Entry == MostRecent)
		{
			MostRecent = Entry->LessRecent;
		}

		Entry->Unlink();
		delete Entry;
	}

public:

	[[nodiscard]] TIterator begin() { return TIterator(*this); }
	[[nodiscard]] TConstIterator begin() const { return TConstIterator(*this); }
	[[nodiscard]] TIterator end() { return TIterator(); }
	[[nodiscard]] TConstIterator end() const { return TConstIterator(); }

private:

	/** Set of entries for fast lookup. */
	TSet<FCacheEntry*, FKeyFuncs> LookupSet;

	/** Least recent item in the cache. */
	FCacheEntry* LeastRecent;

	/** Most recent item in the cache. */
	FCacheEntry* MostRecent;

	/** Maximum number of elements in the cache. */
	int32 MaxNumElements;
};
