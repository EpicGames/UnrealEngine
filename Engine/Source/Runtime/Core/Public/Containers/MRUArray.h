// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UnrealTypeTraits.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Array.h"

/*-----------------------------------------------------------------------------
	MRU array.
-----------------------------------------------------------------------------*/

/**
 * Same as TArray except:
 * - Has an upper limit of the number of items it will store.
 * - Any item that is added to the array is moved to the top.
 */
template<typename T, typename InAllocator = FDefaultAllocator>
class TMRUArray
	: public TArray<T, InAllocator>
{
public:
	typedef TArray<T, InAllocator> Super;

	/** The maximum number of items we can store in this array. */
	int32 MaxItems;

	/**
	 * Constructor.
	 */
	[[nodiscard]] TMRUArray()
		: Super()
	{
		MaxItems = 0;
	}

	[[nodiscard]] TMRUArray(TMRUArray&&) = default;
	[[nodiscard]] TMRUArray(const TMRUArray&) = default;
	TMRUArray& operator=(TMRUArray&&) = default;
	TMRUArray& operator=(const TMRUArray&) = default;

	/**
	 * Adds item to the array. Makes sure that we don't add more than the
	 * limit.
	 *
	 * @param Item Item to add.
	 * @returns Always 0.
	 */
	int32 Add(const T& Item)
	{
		const int32 idx = Super::Add(Item);
		this->Swap(idx, 0);
		CullArray();
		return 0;
	}

	/**
	 * Adds a number of zeroed elements to the array. Makes sure that we don't
	 * add more than the limit.
	 *
	 * @param Count (Optional) A number of elements to add. Default is 0.
	 * @returns Always 0.
	 */
	int32 AddZeroed(int32 Count = 1)
	{
		const int32 idx = Super::AddZeroed(Count);
		this->Swap(idx, 0);
		CullArray();
		return 0;
	}

	/**
	 * Adds unique item to the array. Makes sure that we don't add more than
	 * the limit. If the item existed it will be removed before addition.
	 *
	 * @param Item Element to add.
	 * @returns Always 0.
	 */
	int32 AddUnique(const T& Item)
	{
		// Remove any existing copies of the item.
		this->Remove(Item);
		this->Insert(Item, 0);

		CullArray();

		return 0;
	}

	/**
	 * Makes sure that the array never gets beyond MaxItems in size.
	 */
	void CullArray()
	{
		// 0 = no limit
		if (!MaxItems)
		{
			return;
		}

		while (this->Num() > MaxItems)
		{
			this->RemoveAt(this->Num() - 1, 1);
		}
	}
};



