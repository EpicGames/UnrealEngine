// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Algo/Reverse.h"
#include "Containers/ContainerElementTypeCompatibility.h"
#include "Containers/Set.h"
#include "Containers/SparseSet.h"
#include "Containers/CompactSet.h"
#include "Containers/UnrealString.h"
#include "Misc/AssertionMacros.h"
#include "Misc/StructBuilder.h"
#include "Templates/Function.h"
#include "Templates/Sorting.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "Traits/IsTriviallyRelocatable.h"
#include <type_traits>

#define ExchangeB(A,B) {bool T=A; A=B; B=T;}

/** An initializer type for pairs that's passed to the pair set when adding a new pair. */
template <typename KeyInitType, typename ValueInitType>
class TPairInitializer
{
public:
	std::conditional_t<std::is_rvalue_reference_v<KeyInitType>,   KeyInitType&,   KeyInitType>   Key;
	std::conditional_t<std::is_rvalue_reference_v<ValueInitType>, ValueInitType&, ValueInitType> Value;

	/** Initialization constructor. */
	[[nodiscard]] inline TPairInitializer(KeyInitType InKey, ValueInitType InValue)
		: Key(InKey)
		, Value(InValue)
	{
	}

	/** Implicit conversion to pair initializer. */
	template <typename KeyType, typename ValueType>
	[[nodiscard]] inline TPairInitializer(const TPair<KeyType, ValueType>& Pair)
		: Key(Pair.Key)
		, Value(Pair.Value)
	{
	}

	template <typename KeyType, typename ValueType>
	[[nodiscard]] operator TPair<KeyType, ValueType>() const
	{
		return TPair<KeyType, ValueType>(StaticCast<KeyInitType>(Key), StaticCast<ValueInitType>(Value));
	}
};


/** An initializer type for keys that's passed to the pair set when adding a new key. */
template <typename KeyInitType>
class TKeyInitializer
{
public:
	std::conditional_t<std::is_rvalue_reference_v<KeyInitType>, KeyInitType&, KeyInitType> Key;

	/** Initialization constructor. */
	[[nodiscard]] UE_FORCEINLINE_HINT explicit TKeyInitializer(KeyInitType InKey)
		: Key(InKey)
	{
	}

	template <typename KeyType, typename ValueType>
	[[nodiscard]] operator TPair<KeyType, ValueType>() const
	{
		return TPair<KeyType, ValueType>(StaticCast<KeyInitType>(Key), ValueType());
	}
};

/** Defines how the map's pairs are hashed. */
template<typename KeyType, typename ValueType, bool bInAllowDuplicateKeys>
struct TDefaultMapKeyFuncs : BaseKeyFuncs<TPair<KeyType, ValueType>, KeyType, bInAllowDuplicateKeys>
{
	typedef typename TTypeTraits<KeyType>::ConstPointerType KeyInitType;
	typedef const TPairInitializer<typename TTypeTraits<KeyType>::ConstInitType, typename TTypeTraits<ValueType>::ConstInitType>& ElementInitType;

	[[nodiscard]] static UE_FORCEINLINE_HINT KeyInitType GetSetKey(ElementInitType Element)
	{
		return Element.Key;
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT bool Matches(KeyInitType A, KeyInitType B)
	{
		return A == B;
	}

	template<typename ComparableKey>
	[[nodiscard]] static UE_FORCEINLINE_HINT bool Matches(KeyInitType A, ComparableKey B)
	{
		return A == B;
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key);
	}

	template<typename ComparableKey>
	[[nodiscard]] static UE_FORCEINLINE_HINT uint32 GetKeyHash(ComparableKey Key)
	{
		return GetTypeHash(Key);
	}
};

template<typename KeyType, typename ValueType, bool bInAllowDuplicateKeys>
struct TDefaultMapHashableKeyFuncs : TDefaultMapKeyFuncs<KeyType, ValueType, bInAllowDuplicateKeys>
{
	// Check that the key type is actually hashable
	//
	// If this line fails to compile then your key doesn't have
	// a GetTypeHash() overload.
	using HashabilityCheck = decltype(GetTypeHash(DeclVal<const KeyType>()));
};

template <typename AllocatorType, typename InDerivedType = void>
class TScriptMap;

/**
 * Traits which determine whether or not a type is a TMap.
 */
template <typename T>
inline constexpr bool TIsTMap_V = false;

template <typename T> inline constexpr bool TIsTMap_V<const          T> = TIsTMap_V<T>;
template <typename T> inline constexpr bool TIsTMap_V<      volatile T> = TIsTMap_V<T>;
template <typename T> inline constexpr bool TIsTMap_V<const volatile T> = TIsTMap_V<T>;

template <typename T>
struct TIsTMap
{
	static inline constexpr bool Value = TIsTMap_V<T>;
};

#define UE_TMAP_PREFIX T
#include "Map.h.inl"
#undef UE_TMAP_PREFIX

#define UE_TMAP_PREFIX TSparse
#include "Map.h.inl"
#undef UE_TMAP_PREFIX

#define UE_TMAP_PREFIX TCompact
#include "Map.h.inl"
#undef UE_TMAP_PREFIX

struct FScriptMapLayout
{
	// int32 KeyOffset; // is always at zero offset from the TPair - not stored here
	int32 ValueOffset;

	FScriptSetLayout SetLayout;
};


// Untyped map type for accessing TMap data, like FScriptArray for TArray.
// Must have the same memory representation as a TMap.
template <typename AllocatorType, typename InDerivedType>
class TScriptMap
{
	using DerivedType = std::conditional_t<std::is_void_v<InDerivedType>, TScriptMap, InDerivedType>;

public:
	[[nodiscard]] static constexpr FScriptMapLayout GetScriptLayout(int32 KeySize, int32 KeyAlignment, int32 ValueSize, int32 ValueAlignment)
	{
		FScriptMapLayout Result;

		// TPair<Key, Value>
		FStructBuilder PairStruct;
		int32 KeyOffset = PairStruct.AddMember(KeySize, KeyAlignment);
		Result.ValueOffset = PairStruct.AddMember(ValueSize, ValueAlignment);
		Result.SetLayout = FScriptSet::GetScriptLayout(PairStruct.GetSize(), PairStruct.GetAlignment());

		checkf(KeyOffset == 0, TEXT("The key inside the TPair is expected to be at the start of the struct"));

		return Result;
	}

	[[nodiscard]] TScriptMap()
	{
	}

	///////////////////////////////////////////////////
	// Start - intrusive TOptional<TScriptMap> state //
	///////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TScriptMap;

	[[nodiscard]] explicit TScriptMap(FIntrusiveUnsetOptionalState Tag)
		: Pairs(Tag)
	{
	}
	[[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState Tag) const
	{
		return Pairs == Tag;
	}
	/////////////////////////////////////////////////
	// End - intrusive TOptional<TScriptMap> state //
	/////////////////////////////////////////////////

	[[nodiscard]] bool IsValidIndex(int32 Index) const
	{
		return Pairs.IsValidIndex(Index);
	}

	[[nodiscard]] bool IsEmpty() const
	{
		return Pairs.IsEmpty();
	}

	[[nodiscard]] int32 Num() const
	{
		return Pairs.Num();
	}

	/** @return The number of elements the map can hold before reallocation. */
	[[nodiscard]] int32 Max() const
	{
		return Pairs.Max();
	}

	[[nodiscard]] int32 NumUnchecked() const
	{
		return Pairs.NumUnchecked();
	}

	/** @return The max valid index of the elements in the sparse storage. */
	[[nodiscard]] int32 GetMaxIndex() const
	{
		return Pairs.GetMaxIndex();
	}

	[[nodiscard]] void* GetData(int32 Index, const FScriptMapLayout& Layout)
	{
		return Pairs.GetData(Index, Layout.SetLayout);
	}

	[[nodiscard]] const void* GetData(int32 Index, const FScriptMapLayout& Layout) const
	{
		return Pairs.GetData(Index, Layout.SetLayout);
	}

	void MoveAssign(DerivedType& Other, const FScriptMapLayout& Layout)
	{
		TScriptMap* TypedOther = (TScriptMap*)&Other;

		checkSlow(this != TypedOther);
		Empty(0, Layout);
		Pairs.MoveAssign(TypedOther->Pairs, Layout.SetLayout);
	}

	void Empty(int32 Slack, const FScriptMapLayout& Layout)
	{
		Pairs.Empty(Slack, Layout.SetLayout);
	}

#if UE_USE_COMPACT_SET_AS_DEFAULT
	void RemoveAt(int32 Index, const FScriptMapLayout& Layout, TFunctionRef<uint32 (const void*)> GetKeyHash, TFunctionRef<void (void*)> DestructItem)
	{
		Pairs.RemoveAt(Index, Layout.SetLayout, GetKeyHash, DestructItem);
	}
#else
	void RemoveAt(int32 Index, const FScriptMapLayout& Layout)
	{
		Pairs.RemoveAt(Index, Layout.SetLayout);
	}
#endif

	/**
	 * Adds an uninitialized object to the map.
	 * The map will need rehashing at some point after this call to make it valid.
	 *
	 * @return The index of the added element.
	 */
	int32 AddUninitialized(const FScriptMapLayout& Layout)
	{
		return Pairs.AddUninitialized(Layout.SetLayout);
	}

	void RemoveAtUninitialized(const FScriptMapLayout& Layout, int32 Index)
	{
		Pairs.RemoveAtUninitialized(Layout.SetLayout, Index);
	}

	void CommitLastUninitialized(const FScriptMapLayout& Layout, TFunctionRef<uint32 (const void*)> GetKeyHash)
	{
		Pairs.CommitLastUninitialized(Layout.SetLayout, GetKeyHash);
	}

	void CommitAllUninitialized(const FScriptMapLayout& Layout, TFunctionRef<uint32 (const void*)> GetKeyHash)
	{
		Pairs.CommitAllUninitialized(Layout.SetLayout, GetKeyHash);
	}

	void Rehash(const FScriptMapLayout& Layout, TFunctionRef<uint32(const void*)> GetKeyHash)
	{
		Pairs.Rehash(Layout.SetLayout, GetKeyHash);
	}
	
	/** Finds the associated key, value from hash of Key, rather than linearly searching */
	[[nodiscard]] int32 FindPairIndex(const void* Key, const FScriptMapLayout& MapLayout, TFunctionRef<uint32(const void*)> GetKeyHash, TFunctionRef<bool(const void*, const void*)> KeyEqualityFn) const
	{
		if (Pairs.Num())
		{
			// !unsafe! 'Pairs' is mostly treated as a set of TPair<Key, Value>, so anything in
			// FScriptSet could assume that Key is actually a TPair<Key, Value>, we can hide this
			// complexity from our caller, at least (so their GetKeyHash/EqualityFn is unaware):
			return Pairs.FindIndex(
				Key,
				MapLayout.SetLayout,
				GetKeyHash, // We 'know' that the implementation of Find doesn't call GetKeyHash on anything except Key
				[KeyEqualityFn](const void* InKey, const void* InPair )
				{
					return KeyEqualityFn(InKey, (uint8*)InPair);
				}
			);
		}

		return INDEX_NONE;
	}

	/** Finds the associated value from hash of Key, rather than linearly searching */
	[[nodiscard]] uint8* FindValue(const void* Key, const FScriptMapLayout& MapLayout, TFunctionRef<uint32(const void*)> GetKeyHash, TFunctionRef<bool(const void*, const void*)> KeyEqualityFn)
	{
		int32 FoundIndex = FindPairIndex(Key, MapLayout, GetKeyHash, KeyEqualityFn);
		if (FoundIndex != INDEX_NONE)
		{
			uint8* Result = (uint8*)GetData(FoundIndex, MapLayout) + MapLayout.ValueOffset;
			return Result;
		}

		return nullptr;
	}

	/** Adds the (key, value) pair to the map */
	void Add(
		const void* Key,
		const void* Value,
		const FScriptMapLayout& Layout,
		TFunctionRef<uint32(const void*)> GetKeyHash,
		TFunctionRef<bool(const void*, const void*)> KeyEqualityFn,
		TFunctionRef<void(void*)> KeyConstructAndAssignFn,
		TFunctionRef<void(void*)> ValueConstructAndAssignFn,
		TFunctionRef<void(void*)> ValueAssignFn,
		TFunctionRef<void(void*)> DestructKeyFn,
		TFunctionRef<void(void*)> DestructValueFn)
	{
		Pairs.Add(
			Key,
			Layout.SetLayout,
			GetKeyHash,
			KeyEqualityFn,
			[KeyConstructAndAssignFn, ValueConstructAndAssignFn, Layout](void* NewPair)
		{
			KeyConstructAndAssignFn((uint8*)NewPair);
			ValueConstructAndAssignFn((uint8*)NewPair + Layout.ValueOffset);
		},
			[DestructKeyFn, DestructValueFn, Layout](void* NewPair)
		{
			DestructValueFn((uint8*)NewPair + Layout.ValueOffset);
			DestructKeyFn((uint8*)NewPair);
		}
		);
	}

	/**
	 * Constructs a new key-value pair if key didn't exist
	 *
	 * No need to rehash after calling. The hash table must be properly hashed before calling.
	 *
	 * @return The address to the value, not the pair
	 **/
	void* FindOrAdd(
		const void* Key,
		const FScriptMapLayout& Layout,
		TFunctionRef<uint32(const void*)> GetKeyHash,
		TFunctionRef<bool(const void*, const void*)> KeyEqualityFn,
		TFunctionRef<void(void*, void*)> ConstructPairFn)
	{
		const int32 ValueOffset = Layout.ValueOffset;
		int32 PairIndex = Pairs.FindOrAdd(
			Key,
			Layout.SetLayout,
			GetKeyHash,
			KeyEqualityFn,
			[ConstructPairFn, ValueOffset](void* NewPair)
			{
				ConstructPairFn(NewPair, (uint8*)NewPair + ValueOffset);
			});

		return (uint8*)Pairs.GetData(PairIndex, Layout.SetLayout) + ValueOffset;
	}

private:
	TScriptSet<AllocatorType> Pairs;

	// This function isn't intended to be called, just to be compiled to validate the correctness of the type.
	static void CheckConstraints()
	{
		typedef TScriptMap        ScriptType;
		typedef TMap<int32, int8> RealType;

		// Check that the class footprint is the same
		static_assert(sizeof(ScriptType) == sizeof(RealType), "TScriptMap's size doesn't match TMap");
		static_assert(alignof(ScriptType) == alignof(RealType), "TScriptMap's alignment doesn't match TMap");

		// Check member sizes
		static_assert(sizeof(DeclVal<ScriptType>().Pairs) == sizeof(DeclVal<RealType>().Pairs), "TScriptMap's Pairs member size does not match TMap's");

		// Check member offsets
		static_assert(STRUCT_OFFSET(ScriptType, Pairs) == STRUCT_OFFSET(RealType, Pairs), "TScriptMap's Pairs member offset does not match TMap's");
	}

public:
	// These should really be private, because they shouldn't be called, but there's a bunch of code
	// that needs to be fixed first.
	TScriptMap(const TScriptMap&) { check(false); }
	void operator=(const TScriptMap&) { check(false); }
};


template <typename AllocatorType>
struct TIsZeroConstructType<TScriptMap<AllocatorType>>
{
	enum { Value = true };
};

class FScriptMap : public TScriptMap<FDefaultSetAllocator, FScriptMap>
{
	using Super = TScriptMap<FDefaultSetAllocator, FScriptMap>;

public:
	using Super::Super;

	///////////////////////////////////////////////////
	// Start - intrusive TOptional<FScriptMap> state //
	///////////////////////////////////////////////////
	using IntrusiveUnsetOptionalStateType = FScriptMap;
	/////////////////////////////////////////////////
	// End - intrusive TOptional<FScriptMap> state //
	/////////////////////////////////////////////////
};
