// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef UE_TMAP_PREFIX
#error "Map.h.inl should only be included after defining UE_TMAP_PREFIX"
#endif

#define TMAPBASE UE_JOIN(UE_TMAP_PREFIX, MapBase)
#define TSORTABLEMAPBASE UE_JOIN(UE_TMAP_PREFIX, SortableMapBase)
#define TMAP UE_JOIN(UE_TMAP_PREFIX, Map)
#define TMAPPRIVATEFRIEND UE_JOIN(TMAP,PrivateFriend)
#define TMULTIMAP UE_JOIN(UE_TMAP_PREFIX, MultiMap)
#define TSET UE_JOIN(UE_TMAP_PREFIX, Set)
#define TMAP_STRINGIFY(name) #name

/** 
 * The base class of maps from keys to values.  Implemented using a TSet of key-value pairs with a custom KeyFuncs, 
 * with the same O(1) addition, removal, and finding. 
 *
 * The ByHash() functions are somewhat dangerous but particularly useful in two scenarios:
 * -- Heterogeneous lookup to avoid creating expensive keys like FString when looking up by const TCHAR*.
 *	  You must ensure the hash is calculated in the same way as ElementType is hashed.
 *    If possible put both ComparableKey and ElementType hash functions next to each other in the same header
 *    to avoid bugs when the ElementType hash function is changed.
 * -- Reducing contention around hash tables protected by a lock. It is often important to incur
 *    the cache misses of reading key data and doing the hashing *before* acquiring the lock.
 **/
template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
class TMAPBASE
{
	template <typename OtherKeyType, typename OtherValueType, typename OtherSetAllocator, typename OtherKeyFuncs>
	friend class TMAPBASE;

public:
	UE_DEPRECATED(5.8, "KeyConstPointerType is deprecated - use KeyInitType instead.")
	typedef typename TTypeTraits<KeyType  >::ConstPointerType KeyConstPointerType;
	typedef typename KeyFuncs::KeyInitType                    KeyInitType;
	typedef typename TTypeTraits<ValueType>::ConstInitType    ValueInitType;
	typedef TPair<KeyType, ValueType> ElementType;

protected:
	using ElementSetType = TSET<ElementType, KeyFuncs, SetAllocator>;

	[[nodiscard]] constexpr TMAPBASE() = default;
	[[nodiscard]] explicit consteval TMAPBASE(EConstEval)
	: Pairs(ConstEval)
	{
	} 
	[[nodiscard]] TMAPBASE(TMAPBASE&&) = default;
	[[nodiscard]] TMAPBASE(const TMAPBASE&) = default;
	TMAPBASE& operator=(TMAPBASE&&) = default;
	TMAPBASE& operator=(const TMAPBASE&) = default;

	/** Constructor for moving elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	[[nodiscard]] TMAPBASE(TMAPBASE<KeyType, ValueType, OtherSetAllocator, KeyFuncs>&& Other)
	: Pairs(MoveTemp(Other.Pairs))
	{
	}

	/** Constructor for copying elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	[[nodiscard]] TMAPBASE(const TMAPBASE<KeyType, ValueType, OtherSetAllocator, KeyFuncs>& Other)
	: Pairs(Other.Pairs)
	{
	}

	~TMAPBASE()
	{
		UE_STATIC_ASSERT_WARN(TIsTriviallyRelocatable_V<KeyType> && TIsTriviallyRelocatable_V<ValueType>, "TMapBase can only be used with trivially relocatable types");
	}

	/////////////////////////////////////////////////
	// Start - intrusive TOptional<TMAPBASE> state //
	/////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TMAPBASE;

	[[nodiscard]] explicit TMAPBASE(FIntrusiveUnsetOptionalState Tag)
	: Pairs(Tag)
	{
	}
	[[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState Tag) const
	{
		return Pairs == Tag;
	}
	///////////////////////////////////////////////
	// End - intrusive TOptional<TMAPBASE> state //
	///////////////////////////////////////////////

	/** Assignment operator for moving elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	TMAPBASE& operator=(TMAPBASE<KeyType, ValueType, OtherSetAllocator, KeyFuncs>&& Other)
	{
		Pairs = MoveTemp(Other.Pairs);
		return *this;
	}

	/** Assignment operator for copying elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	TMAPBASE& operator=(const TMAPBASE<KeyType, ValueType, OtherSetAllocator, KeyFuncs>& Other)
	{
		Pairs = Other.Pairs;
		return *this;
	}

public:

	/**
	 * Compare this map with another for equality. Does not make any assumptions about Key order.
	 * NOTE: this might be a candidate for operator== but it was decided to make it an explicit function
	 *  since it can potentially be quite slow.
	 *
	 * @param Other The other map to compare against
	 * @returns True if both this and Other contain the same keys with values that compare ==
	 */
	[[nodiscard]] bool OrderIndependentCompareEqual(const TMAPBASE& Other) const
	{
		// first check counts (they should be the same obviously)
		if (Num() != Other.Num())
		{
			return false;
		}

		// since we know the counts are the same, we can just iterate one map and check for existence in the other
		for (typename ElementSetType::TConstIterator It(Pairs); It; ++It)
		{
			const ValueType* BVal = Other.Find(It->Key);
			if (BVal == nullptr)
			{
				return false;
			}
			if (!(*BVal == It->Value))
			{
				return false;
			}
		}

		// all fields in A match B and A and B's counts are the same (so there can be no fields in B not in A)
		return true;
	}

	/**
	 * Removes all elements from the map.
	 *
	 * This method potentially leaves space allocated for an expected
	 * number of elements about to be added.
	 *
	 * @param ExpectedNumElements The number of elements about to be added to the set.
	 */
	UE_FORCEINLINE_HINT void Empty(int32 ExpectedNumElements = 0)
	{
		Pairs.Empty(ExpectedNumElements);
	}

	/** Efficiently empties out the map but preserves all allocations and capacities */
	UE_FORCEINLINE_HINT void Reset()
	{
		Pairs.Reset();
	}

	/** Shrinks the pair set to avoid slack. */
	UE_FORCEINLINE_HINT void Shrink()
	{
		Pairs.Shrink();
	}

	/** Compacts the pair set to remove holes */
	UE_FORCEINLINE_HINT void Compact()
	{
		Pairs.Compact();
	}

	/** Compacts the pair set to remove holes. Does not change the iteration order of the elements. */
	UE_FORCEINLINE_HINT void CompactStable() requires (&ElementSetType::CompactStable != nullptr)
	{
		Pairs.CompactStable();
	}

	/** Preallocates enough memory to contain Number elements */
	UE_FORCEINLINE_HINT void Reserve(int32 Number)
	{
		Pairs.Reserve(Number);
	}

	/**
	 * Returns true if the map is empty and contains no elements. 
	 *
	 * @returns True if the map is empty.
	 * @see Num
	 */
	[[nodiscard]] bool IsEmpty() const
	{
		return Pairs.IsEmpty();
	}

	/** @return The number of elements in the map. */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Num() const
	{
		return Pairs.Num();
	}

	/** @return The number of elements the map can hold before reallocation. */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Max() const
	{
		return Pairs.Max();
	}

	/** @return The non-inclusive maximum index of elements in the map. */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 GetMaxIndex() const
	{
		return Pairs.GetMaxIndex();
	}

	/**
	 * Checks whether an element id is valid.
	 * @param Id - The element id to check.
	 * @return true if the element identifier refers to a valid element in this map.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValidId(FSetElementId Id) const
	{
		return Pairs.IsValidId(Id);
	}

	/** Return a mapped pair by internal identifier. Element must be valid (see @IsValidId). */
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& Get(FSetElementId Id)
	{
		return Pairs[Id];
	}

	/** Return a mapped pair by internal identifier.  Element must be valid (see @IsValidId).*/
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType& Get(FSetElementId Id) const
	{
		return Pairs[Id];
	}

	/**
	 * Get the unique keys contained within this map.
	 *
	 * @param OutKeys Upon return, contains the set of unique keys in this map.
	 * @return The number of unique keys in the map.
	 */
	template<typename Allocator> int32 GetKeys(TArray<KeyType, Allocator>& OutKeys) const
	{
		OutKeys.Reset();

		TSet<KeyType> VisitedKeys;
		VisitedKeys.Reserve(Num());

		// Presize the array if we know there are supposed to be no duplicate keys
		if constexpr (!KeyFuncs::bAllowDuplicateKeys)
		{
			OutKeys.Reserve(Num());
		}

		for (typename ElementSetType::TConstIterator It(Pairs); It; ++It)
		{
			// Even if bAllowDuplicateKeys is false, we still want to filter for duplicate
			// keys due to maps with keys that can be invalidated (UObjects, TWeakObj, etc.)
			if (!VisitedKeys.Contains(It->Key))
			{
				OutKeys.Add(It->Key);
				VisitedKeys.Add(It->Key);
			}
		}

		return OutKeys.Num();
	}

	/**
	 * Get the unique keys contained within this map.
	 *
	 * @param OutKeys Upon return, contains the set of unique keys in this map.
	 * @return The number of unique keys in the map.
	 */
	template<typename InSetKeyFuncs, typename InSetAllocator> int32 GetKeys(TSet<KeyType, InSetKeyFuncs, InSetAllocator>& OutKeys) const
	{
		OutKeys.Reset();
		
		// Presize the set if we know there are supposed to be no duplicate keys
		if (!KeyFuncs::bAllowDuplicateKeys)
		{
			OutKeys.Reserve(Num());
		}

		for (typename ElementSetType::TConstIterator It(Pairs); It; ++It)
		{
			OutKeys.Add(It->Key);
		}

		return OutKeys.Num();
	}

	/** 
	 * Helper function to return the amount of memory allocated by this container .
	 * Only returns the size of allocations made directly by the container, not the elements themselves.
	 *
	 * @return Number of bytes allocated by this container.
	 * @see CountBytes
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT SIZE_T GetAllocatedSize() const
	{
		return Pairs.GetAllocatedSize();
	}

	/**
	 * Track the container's memory use through an archive.
	 *
	 * @param Ar The archive to use.
	 * @see GetAllocatedSize
	 */
	UE_FORCEINLINE_HINT void CountBytes(FArchive& Ar) const
	{
		Pairs.CountBytes(Ar);
	}

	/**
	 * Set the value associated with a key.
	 *
	 * @param InKey The key to associate the value with.
	 * @param InValue The value to associate with the key.
	 * @return A reference to the value as stored in the map. The reference is only valid until the next change to any key in the map.
	 */
	UE_FORCEINLINE_HINT ValueType& Add(const KeyType&  InKey, const ValueType&  InValue) { return Emplace(InKey, InValue); }
	UE_FORCEINLINE_HINT ValueType& Add(const KeyType&  InKey,		ValueType&& InValue) { return Emplace(InKey, MoveTempIfPossible(InValue)); }
	UE_FORCEINLINE_HINT ValueType& Add(		 KeyType&& InKey, const ValueType&  InValue) { return Emplace(MoveTempIfPossible(InKey), InValue); }
	UE_FORCEINLINE_HINT ValueType& Add(		 KeyType&& InKey,		ValueType&& InValue) { return Emplace(MoveTempIfPossible(InKey), MoveTempIfPossible(InValue)); }

	/** See Add() and class documentation section on ByHash() functions */
	UE_FORCEINLINE_HINT ValueType& AddByHash(uint32 KeyHash, const KeyType&  InKey, const ValueType&  InValue) { return EmplaceByHash(KeyHash, InKey, InValue); }
	UE_FORCEINLINE_HINT ValueType& AddByHash(uint32 KeyHash, const KeyType&  InKey,		  ValueType&& InValue) { return EmplaceByHash(KeyHash, InKey, MoveTempIfPossible(InValue)); }
	UE_FORCEINLINE_HINT ValueType& AddByHash(uint32 KeyHash,	   KeyType&& InKey, const ValueType&  InValue) { return EmplaceByHash(KeyHash, MoveTempIfPossible(InKey), InValue); }
	UE_FORCEINLINE_HINT ValueType& AddByHash(uint32 KeyHash,	   KeyType&& InKey,		  ValueType&& InValue) { return EmplaceByHash(KeyHash, MoveTempIfPossible(InKey), MoveTempIfPossible(InValue)); }

	/**
	 * Set a default value associated with a key.
	 *
	 * @param InKey The key to associate the value with.
	 * @return A reference to the value as stored in the map. The reference is only valid until the next change to any key in the map.
	 */
	UE_FORCEINLINE_HINT ValueType& Add(const KeyType&  InKey) { return Emplace(InKey); }
	UE_FORCEINLINE_HINT ValueType& Add(		 KeyType&& InKey) { return Emplace(MoveTempIfPossible(InKey)); }

	/** See Add() and class documentation section on ByHash() functions */
	UE_FORCEINLINE_HINT ValueType& AddByHash(uint32 KeyHash, const KeyType&  InKey) { return EmplaceByHash(KeyHash, InKey); }
	UE_FORCEINLINE_HINT ValueType& AddByHash(uint32 KeyHash,	   KeyType&& InKey)	{ return EmplaceByHash(KeyHash, MoveTempIfPossible(InKey)); }

	/**
	 * Set the value associated with a key.
	 *
	 * @param InKeyValue A Tuple containing the Key and Value to associate together
	 * @return A reference to the value as stored in the map. The reference is only valid until the next change to any key in the map.
	 */
	UE_FORCEINLINE_HINT ValueType& Add(const TTuple<KeyType, ValueType>&  InKeyValue) { return Emplace(InKeyValue.Key, InKeyValue.Value); }
	UE_FORCEINLINE_HINT ValueType& Add(		 TTuple<KeyType, ValueType>&& InKeyValue) { return Emplace(MoveTempIfPossible(InKeyValue.Key), MoveTempIfPossible(InKeyValue.Value)); }

	/**
	 * Sets the value associated with a key.
	 *
	 * @param InKey The key to associate the value with.
	 * @param InValue The value to associate with the key.
	 * @return A reference to the value as stored in the map. The reference is only valid until the next change to any key in the map.
	 */
	template <typename InitKeyType = KeyType, typename InitValueType = ValueType>
	ValueType& Emplace(InitKeyType&& InKey, InitValueType&& InValue)
	{
		const FSetElementId PairId = Pairs.Emplace(TPairInitializer<InitKeyType&&, InitValueType&&>(Forward<InitKeyType>(InKey), Forward<InitValueType>(InValue)));

		return Pairs[PairId].Value;
	}

	/** See Emplace() and class documentation section on ByHash() functions */
	template <typename InitKeyType = KeyType, typename InitValueType = ValueType>
	ValueType& EmplaceByHash(uint32 KeyHash, InitKeyType&& InKey, InitValueType&& InValue)
	{
		const FSetElementId PairId = Pairs.EmplaceByHash(KeyHash, TPairInitializer<InitKeyType&&, InitValueType&&>(Forward<InitKeyType>(InKey), Forward<InitValueType>(InValue)));

		return Pairs[PairId].Value;
	}

	/**
	 * Set a default value associated with a key.
	 *
	 * @param InKey The key to associate the value with.
	 * @return A reference to the value as stored in the map. The reference is only valid until the next change to any key in the map.
	 */
	template <typename InitKeyType = KeyType>
	ValueType& Emplace(InitKeyType&& InKey)
	{
		const FSetElementId PairId = Pairs.Emplace(TKeyInitializer<InitKeyType&&>(Forward<InitKeyType>(InKey)));

		return Pairs[PairId].Value;
	}

	/** See Emplace() and class documentation section on ByHash() functions */
	template <typename InitKeyType = KeyType>
	ValueType& EmplaceByHash(uint32 KeyHash, InitKeyType&& InKey)
	{
		const FSetElementId PairId = Pairs.EmplaceByHash(KeyHash, TKeyInitializer<InitKeyType&&>(Forward<InitKeyType>(InKey)));

		return Pairs[PairId].Value;
	}

	/**
	 * Remove all value associations for a key.
	 *
	 * @param InKey The key to remove associated values for.
	 * @return The number of values that were associated with the key.
	 */
	inline int32 Remove(KeyInitType InKey)
	{
		const int32 NumRemovedPairs = Pairs.Remove(InKey);
		return NumRemovedPairs;
	}

	inline int32 RemoveStable(KeyInitType InKey)
	{
		const int32 NumRemovedPairs = Pairs.RemoveStable(InKey);
		return NumRemovedPairs;
	}

	/** See Remove() and class documentation section on ByHash() functions */
	template<typename ComparableKey>
	inline int32 RemoveByHash(uint32 KeyHash, const ComparableKey& Key)
	{
		const int32 NumRemovedPairs = Pairs.RemoveByHash(KeyHash, Key);
		return NumRemovedPairs;
	}

	/**
	 * Removes the element at the specified index. The caller has to ensure that the index is valid.
	 * @param Id The index of the element to remove.
	 */
	UE_FORCEINLINE_HINT void Remove(FSetElementId Id)
	{
		Pairs.Remove(Id);
	}

	/**
	 * Find the key associated with the specified value.
	 *
	 * The time taken is O(N) in the number of pairs.
	 *
	 * @param Value The value to search for
	 * @return A pointer to the key associated with the specified value,
	 *     or nullptr if the value isn't contained in this map. The pointer
	 *     is only valid until the next change to any key in the map.
	 */
	[[nodiscard]] const KeyType* FindKey(ValueInitType Value) const
	{
		for (typename ElementSetType::TConstIterator PairIt(Pairs); PairIt; ++PairIt)
		{
			if (PairIt->Value == Value)
			{
				return &PairIt->Key;
			}
		}
		return nullptr;
	}

	/**
	 * Filters the elements in the map based on a predicate functor.
	 *
	 * @param Pred The functor to apply to each element.
	 * @returns TMAP with the same type as this object which contains
	 *          the subset of elements for which the functor returns true.
	 */
	template <typename Predicate>
	[[nodiscard]] TMAP<KeyType, ValueType, SetAllocator, KeyFuncs> FilterByPredicate(Predicate Pred) const
	{
		TMAP<KeyType, ValueType, SetAllocator, KeyFuncs> FilterResults;
		FilterResults.Reserve(Pairs.Num());
		for (const ElementType& Pair : Pairs)
		{
			if (Pred(Pair))
			{
				FilterResults.Add(Pair);
			}
		}
		return FilterResults;
	}

	/**
	 * Find the value associated with a specified key.
	 *
	 * @param Key The key to search for.
	 * @return A pointer to the value associated with the specified key, or nullptr if the key isn't contained in this map.  The pointer
	 *			is only valid until the next change to any key in the map.
	 */
	[[nodiscard]] inline ValueType* Find(KeyInitType Key)
	{
		if (auto* Pair = Pairs.Find(Key))
		{
			return &Pair->Value;
		}

		return nullptr;
	}
	[[nodiscard]] UE_FORCEINLINE_HINT const ValueType* Find(KeyInitType Key) const
	{
		return const_cast<TMAPBASE*>(this)->Find(Key);
	}

	/** See Find() and class documentation section on ByHash() functions */
	template<typename ComparableKey>
	[[nodiscard]] inline ValueType* FindByHash(uint32 KeyHash, const ComparableKey& Key)
	{
		if (auto* Pair = Pairs.FindByHash(KeyHash, Key))
		{
			return &Pair->Value;
		}

		return nullptr;
	}
	template<typename ComparableKey>
	[[nodiscard]] UE_FORCEINLINE_HINT const ValueType* FindByHash(uint32 KeyHash, const ComparableKey& Key) const
	{
		return const_cast<TMAPBASE*>(this)->FindByHash(KeyHash, Key);
	}

	template<typename ComparableKey>
	[[nodiscard]] inline ValueType& FindByHashChecked(uint32 KeyHash, const ComparableKey& Key)
	{
		auto* Pair = Pairs.FindByHash(KeyHash, Key);
		check(Pair != nullptr);
		return Pair->Value;
	}
	template<typename ComparableKey>
	[[nodiscard]] UE_FORCEINLINE_HINT const ValueType& FindByHashChecked(uint32 KeyHash, const ComparableKey& Key) const
	{
		return const_cast<TMAPBASE*>(this)->FindByHashChecked(KeyHash, Key);
	}

	/**
	 * Finds the index of the first element that is assigned to the specified key.
	 * The returned index is only valid until the map is changed.
	 *
	 * @param Key The key to search for.
	 */
	UE_FORCEINLINE_HINT FSetElementId FindId(KeyInitType Key) const
	{
		return Pairs.FindId(Key);
	}

	/** See FindId() and class documentation section on ByHash() functions */
	template<typename ComparableKey>
	UE_FORCEINLINE_HINT FSetElementId FindIdByHash(uint32 KeyHash, const ComparableKey& Key) const
	{
		return Pairs.FindIdByHash(KeyHash, Key);
	}

private:
	[[nodiscard]] UE_FORCEINLINE_HINT static uint32 HashKey(const KeyType& Key)
	{
		return KeyFuncs::GetKeyHash(Key);
	}

	/**
	 * Find the value associated with a specified key, or if none exists, 
	 * adds a value using the default constructor.
	 *
	 * @param Key The key to search for.
	 * @return A reference to the value associated with the specified key.
	 */
	template <typename InitKeyType>
	[[nodiscard]] ValueType& FindOrAddImpl(uint32 KeyHash, InitKeyType&& Key)
	{
		if (auto* Pair = Pairs.FindByHash(KeyHash, Key))
		{
			return Pair->Value;
		}

		return AddByHash(KeyHash, Forward<InitKeyType>(Key));
	}

	/**
	 * Find the value associated with a specified key, or if none exists,
	 * adds the value.
	 *
	 * @param Key The key to search for.
	 * @param Value The value to associate with the key.
	 * @return A reference to the value associated with the specified key.
	 */
	template <typename InitKeyType, typename InitValueType>
	[[nodiscard]] ValueType& FindOrAddImpl(uint32 KeyHash, InitKeyType&& Key, InitValueType&& Value)
	{
		if (auto* Pair = Pairs.FindByHash(KeyHash, Key))
		{
			return Pair->Value;
		}

		return AddByHash(KeyHash, Forward<InitKeyType>(Key), Forward<InitValueType>(Value));
	}

public:

	/**
	 * Find the value associated with a specified key, or if none exists, 
	 * adds a value using the default constructor.
	 *
	 * @param Key The key to search for.
	 * @return A reference to the value associated with the specified key.
	 */
	UE_FORCEINLINE_HINT ValueType& FindOrAdd(const KeyType&  Key) { return FindOrAddImpl(HashKey(Key),					  Key); }
	UE_FORCEINLINE_HINT ValueType& FindOrAdd(      KeyType&& Key) { return FindOrAddImpl(HashKey(Key), MoveTempIfPossible(Key)); }

	/** See FindOrAdd() and class documentation section on ByHash() functions */
	UE_FORCEINLINE_HINT ValueType& FindOrAddByHash(uint32 KeyHash, const KeyType&  Key) { return FindOrAddImpl(KeyHash,                    Key); }
	UE_FORCEINLINE_HINT ValueType& FindOrAddByHash(uint32 KeyHash,       KeyType&& Key) { return FindOrAddImpl(KeyHash, MoveTempIfPossible(Key)); }

	/**
	 * Find the value associated with a specified key, or if none exists, 
	 * adds the value.
	 *
	 * @param Key The key to search for.
	 * @param Value The value to associate with the key.
	 * @return A reference to the value associated with the specified key.
	 */
	UE_FORCEINLINE_HINT ValueType& FindOrAdd(const KeyType&  Key, const ValueType&  Value) { return FindOrAddImpl(HashKey(Key),						Key,                    Value  ); }
	UE_FORCEINLINE_HINT ValueType& FindOrAdd(const KeyType&  Key, ValueType&&       Value) { return FindOrAddImpl(HashKey(Key),						Key, MoveTempIfPossible(Value) ); }
	UE_FORCEINLINE_HINT ValueType& FindOrAdd(      KeyType&& Key, const ValueType&  Value) { return FindOrAddImpl(HashKey(Key), MoveTempIfPossible(Key),                    Value  ); }
	UE_FORCEINLINE_HINT ValueType& FindOrAdd(      KeyType&& Key, ValueType&&       Value) { return FindOrAddImpl(HashKey(Key), MoveTempIfPossible(Key), MoveTempIfPossible(Value) ); }

	/** See FindOrAdd() and class documentation section on ByHash() functions */
	UE_FORCEINLINE_HINT ValueType& FindOrAddByHash(uint32 KeyHash, const KeyType&  Key, const ValueType&  Value) { return FindOrAddImpl(KeyHash,                     Key,                    Value); }
	UE_FORCEINLINE_HINT ValueType& FindOrAddByHash(uint32 KeyHash, const KeyType&  Key,       ValueType&& Value) { return FindOrAddImpl(KeyHash,                     Key, MoveTempIfPossible(Value)); }
	UE_FORCEINLINE_HINT ValueType& FindOrAddByHash(uint32 KeyHash,       KeyType&& Key, const ValueType&  Value) { return FindOrAddImpl(KeyHash, MoveTempIfPossible(Key),                    Value); }
	UE_FORCEINLINE_HINT ValueType& FindOrAddByHash(uint32 KeyHash,       KeyType&& Key,       ValueType&& Value) { return FindOrAddImpl(KeyHash, MoveTempIfPossible(Key), MoveTempIfPossible(Value)); }

	/**
	 * Find a reference to the value associated with a specified key.
	 *
	 * @param Key The key to search for.
	 * @return The value associated with the specified key, or triggers an assertion if the key does not exist.
	 */
	[[nodiscard]] inline const ValueType& FindChecked(KeyInitType Key) const
	{
		const auto* Pair = Pairs.Find(Key);
		check(Pair != nullptr);
		return Pair->Value;
	}

	/**
	 * Find a reference to the value associated with a specified key.
	 *
	 * @param Key The key to search for.
	 * @return The value associated with the specified key, or triggers an assertion if the key does not exist.
	 */
	[[nodiscard]] inline ValueType& FindChecked(KeyInitType Key)
	{
		auto* Pair = Pairs.Find(Key);
		check(Pair != nullptr);
		return Pair->Value;
	}

	/**
	 * Find the value associated with a specified key.
	 *
	 * @param Key The key to search for.
	 * @return The value associated with the specified key, or the default value for the ValueType if the key isn't contained in this map.
	 */
	[[nodiscard]] inline ValueType FindRef(KeyInitType Key) const
	{
		if (const auto* Pair = Pairs.Find(Key))
		{
			return Pair->Value;
		}

		return ValueType();
	}

	/**
	 * Find the value associated with a specified key.
	 *
	 * @param Key The key to search for.
	 * @param DefaultValue The fallback value if the key is not found.
	 * @return The value associated with the specified key, or DefaultValue if the key isn't contained in this map.
	 */
	[[nodiscard]] inline ValueType FindRef(KeyInitType Key, ValueType DefaultValue) const
	{
		if (const auto* Pair = Pairs.Find(Key))
		{
			return Pair->Value;
		}

		return DefaultValue;
	}

	/**
	 * Finds any pair in the map and returns a pointer to it.
	 * Callers should not depend on particular patterns in the behaviour of this function.
	 * @return A pointer to an arbitrary pair, or nullptr if the container is empty.
	 */
	[[nodiscard]] ElementType* FindArbitraryElement()
	{
		// The goal of this function is to be fast, and so the implementation may be improved at any time even if it gives different results.

		return Pairs.FindArbitraryElement();
	}
	[[nodiscard]] const ElementType* FindArbitraryElement() const
	{
		return const_cast<TMAPBASE*>(this)->FindArbitraryElement();
	}

	/**
	 * Check if map contains the specified key.
	 *
	 * @param Key The key to check for.
	 * @return true if the map contains the key.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool Contains(KeyInitType Key) const
	{
		return Pairs.Contains(Key);
	}

	/** See Contains() and class documentation section on ByHash() functions */
	template<typename ComparableKey>
	[[nodiscard]] UE_FORCEINLINE_HINT bool ContainsByHash(uint32 KeyHash, const ComparableKey& Key) const
	{
		return Pairs.ContainsByHash(KeyHash, Key);
	}

	/** Copy the key/value pairs in this map into an array. */
	[[nodiscard]] TArray<ElementType> Array() const
	{
		return Pairs.Array();
	}

	/**
	 * Generate an array from the keys in this map.
	 *
	 * @param OutArray Will contain the collection of keys.
	 */
	template<typename Allocator> void GenerateKeyArray(TArray<KeyType, Allocator>& OutArray) const
	{
		OutArray.Empty(Pairs.Num());
		for (typename ElementSetType::TConstIterator PairIt(Pairs); PairIt; ++PairIt)
		{
			OutArray.Add(PairIt->Key);
		}
	}

	/**
	 * Generate an array from the values in this map.
	 *
	 * @param OutArray Will contain the collection of values.
	 */
	template<typename Allocator> void GenerateValueArray(TArray<ValueType, Allocator>& OutArray) const
	{
		OutArray.Empty(Pairs.Num());
		for (typename ElementSetType::TConstIterator PairIt(Pairs); PairIt; ++PairIt)
		{
			OutArray.Add(PairIt->Value);
		}
	}

	/**
	 * Describes the map's contents through an output device.
	 *
	 * @param Ar The output device to describe the map's contents through.
	 */
	void Dump(FOutputDevice& Ar)
	{
		Pairs.Dump(Ar);
	}

protected:
	/** The base of TMAPBASE iterators. */
	template<bool bConst, bool bKeyIterator>
	class TCommonBaseIterator
	{
	private:
		using BaseIteratorType = std::conditional_t<bConst, typename ElementSetType::TConstIterator, typename ElementSetType::TIterator>;
		using KeyIteratorType = std::conditional_t<bConst, typename ElementSetType::TConstKeyIterator, typename ElementSetType::TKeyIterator>;

	protected:
		using MapType = std::conditional_t<bConst, const TMAPBASE, TMAPBASE>;
		using PairType = std::conditional_t<bConst, const typename ElementSetType::ElementType, typename ElementSetType::ElementType>;

		using PairItType = std::conditional_t<bKeyIterator, KeyIteratorType, BaseIteratorType>;

		using ItKeyType = std::conditional_t<bConst, const KeyType, KeyType>;
		using ItValueType = std::conditional_t<bConst, const ValueType, ValueType>;

	protected:
		[[nodiscard]] explicit UE_FORCEINLINE_HINT TCommonBaseIterator(const PairItType& InElementIt)
		: PairIt(InElementIt)
		{
		}

	public:

		UE_FORCEINLINE_HINT TCommonBaseIterator& operator++()
		{
			++PairIt;
			return *this;
		}

		/** conversion to "bool" returning true if the iterator is valid. */
		[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
		{ 
			return !!PairIt; 
		}
		/** inverse of the "bool" operator */
		[[nodiscard]] UE_FORCEINLINE_HINT bool operator !() const
		{
			return !(bool)*this;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TCommonBaseIterator& Rhs) const
		{
			return PairIt == Rhs.PairIt;
		}
		[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TCommonBaseIterator& Rhs) const
		{
			return PairIt != Rhs.PairIt;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT ItKeyType& Key() const
		{
			return PairIt->Key;
		}
		[[nodiscard]] UE_FORCEINLINE_HINT ItValueType& Value() const
		{
			return PairIt->Value;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT FSetElementId GetId() const
		{
			return PairIt.GetId();
		}

		[[nodiscard]] UE_FORCEINLINE_HINT PairType& operator*() const
		{
			return *PairIt;
		}
		[[nodiscard]] UE_FORCEINLINE_HINT PairType* operator->() const
		{
			return &*PairIt;
		}

	protected:
		PairItType PairIt;
	};
	
	/** The base type of iterators that iterate over the values associated with a specified key. */
	template<bool bConst>
	class TBaseIterator : public TCommonBaseIterator<bConst, false>
	{
	private:
		using Super = TCommonBaseIterator<bConst, false>;
		using MapType = typename Super::MapType;
		using PairItType = typename Super::PairItType;

	public:
		/** Initialization constructor. */
		[[nodiscard]] explicit UE_FORCEINLINE_HINT TBaseIterator(MapType& Map)
		: Super(PairItType(Map.Pairs))
		{
		}
	};

	/** The base type of iterators that iterate over the values associated with a specified key. */
	template<bool bConst>
	class TBaseKeyIterator : public TCommonBaseIterator<bConst, true>
	{
	private:
		using Super = TCommonBaseIterator<bConst, true>;
		using MapType = typename Super::MapType;
		using PairItType = typename Super::PairItType;

	public:
		using KeyArgumentType = typename PairItType::KeyArgumentType;

		/** Initialization constructor. */
		[[nodiscard]] explicit UE_FORCEINLINE_HINT TBaseKeyIterator(MapType& Map, KeyArgumentType InKey)
		: Super(PairItType(Map.Pairs, InKey))
		{
		}
	};

	/** A set of the key-value pairs in the map. */
	ElementSetType Pairs;

public:
	void WriteMemoryImage(FMemoryImageWriter& Writer) const
	{
		Pairs.WriteMemoryImage(Writer);
	}

	void CopyUnfrozen(const FMemoryUnfreezeContent& Context, void* Dst) const
	{
		Pairs.CopyUnfrozen(Context, Dst);
	}

	static void AppendHash(const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		ElementSetType::AppendHash(LayoutParams, Hasher);
	}

	/** Map iterator. */
	class TIterator : public TBaseIterator<false>
	{
	private:
		using Super = TBaseIterator<false>;

	public:
		/** Initialization constructor. */
		[[nodiscard]] explicit inline TIterator(TMAPBASE& InMap)
		: Super(InMap)
		, Map(InMap)
		, bElementsHaveBeenRemoved(false)
		, bRequiresRehashOnRemoval(false)
		{
		}

		UE_DEPRECATED(5.8, "TSet::Relax has been deprecated. Memory updates post iteration should be handled manually as necessary.")
		[[nodiscard]] explicit inline TIterator(TMAPBASE& InMap, bool bInRequiresRehashOnRemoval /* = false */)
		: Super(InMap.Pairs.CreateIterator())
		, Map(InMap)
		, bElementsHaveBeenRemoved(false)
		, bRequiresRehashOnRemoval(bInRequiresRehashOnRemoval)
		{
		}

		/** Destructor. */
		inline ~TIterator()
		{
		    if constexpr (requires { Map.Pairs.Relax(); })
		    {
			    if (bElementsHaveBeenRemoved && bRequiresRehashOnRemoval)
			    {
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					Map.Pairs.Relax();
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
			    }
		    }
		}

		/** Removes the current pair from the map without losing the iteration position.
		 * Increment before using the iterator again, but after that
		 * it will point at the element that was after the removed element.
		 */
		inline void RemoveCurrent()
		{
			this->PairIt.RemoveCurrent();
			bElementsHaveBeenRemoved = true;
		}

	private:
		TMAPBASE& Map;
		bool      bElementsHaveBeenRemoved;
		bool      bRequiresRehashOnRemoval;
	};

	/** Iterates over values associated with a specified key in a map. */
	class TKeyIterator : public TBaseKeyIterator<false>
	{
	private:
		using Super = TBaseKeyIterator<false>;

	public:
		using Super::Super;

		/** Removes the current key-value pair from the map. */
		UE_FORCEINLINE_HINT void RemoveCurrent()
		{
			this->PairIt.RemoveCurrent();
		}
	};

	/** Const map iterator. */
	using TConstIterator = TBaseIterator<true>;
	using TConstKeyIterator = TBaseKeyIterator<true>;

	using TRangedForIterator = typename ElementSetType::TRangedForIterator;
	using TRangedForConstIterator = typename ElementSetType::TRangedForConstIterator;

	/** Creates an iterator over all the pairs in this map */
	[[nodiscard]] UE_FORCEINLINE_HINT TIterator CreateIterator()
	{
		return TIterator(*this);
	}

	/** Creates a const iterator over all the pairs in this map */
	[[nodiscard]] UE_FORCEINLINE_HINT TConstIterator CreateConstIterator() const
	{
		return TConstIterator(*this);
	}

	/** Creates an iterator over the values associated with a specified key in a map */
	[[nodiscard]] UE_FORCEINLINE_HINT TKeyIterator CreateKeyIterator(typename TKeyIterator::KeyArgumentType InKey)
	{
		return TKeyIterator(*this, InKey);
	}

	/** Creates a const iterator over the values associated with a specified key in a map */
	[[nodiscard]] UE_FORCEINLINE_HINT TConstKeyIterator CreateConstKeyIterator(typename TConstKeyIterator::KeyArgumentType InKey) const
	{
		return TConstKeyIterator(*this, InKey);
	}

	friend struct TMAPPRIVATEFRIEND;

public:
	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForIterator      begin() { return TRangedForIterator(Pairs.begin()); }
	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForConstIterator begin() const { return TRangedForConstIterator(Pairs.begin()); }
	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForIterator      end() { return TRangedForIterator(Pairs.end()); }
	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForConstIterator end() const { return TRangedForConstIterator(Pairs.end()); }

	// Maps are deliberately prevented from being hashed or compared, because this would hide potentially major performance problems behind default operations.
	friend uint32 GetTypeHash(const TMAPBASE& Map) = delete;
	friend bool operator==(const TMAPBASE&, const TMAPBASE&) = delete;
	friend bool operator!=(const TMAPBASE&, const TMAPBASE&) = delete;
};

/** The base type of sortable maps. */
template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
class TSORTABLEMAPBASE : public TMAPBASE<KeyType, ValueType, SetAllocator, KeyFuncs>
{
protected:
	typedef TMAPBASE<KeyType, ValueType, SetAllocator, KeyFuncs> Super;

	[[nodiscard]] constexpr TSORTABLEMAPBASE() = default;
	[[nodiscard]] explicit consteval TSORTABLEMAPBASE(EConstEval)
	: Super(ConstEval)
	{
	}
	[[nodiscard]] TSORTABLEMAPBASE(TSORTABLEMAPBASE&&) = default;
	[[nodiscard]] TSORTABLEMAPBASE(const TSORTABLEMAPBASE&) = default;
	TSORTABLEMAPBASE& operator=(TSORTABLEMAPBASE&&) = default;
	TSORTABLEMAPBASE& operator=(const TSORTABLEMAPBASE&) = default;

	/** Constructor for moving elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	[[nodiscard]] TSORTABLEMAPBASE(TSORTABLEMAPBASE<KeyType, ValueType, OtherSetAllocator, KeyFuncs>&& Other)
	: Super(MoveTemp(Other))
	{
	}

	/** Constructor for copying elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	[[nodiscard]] TSORTABLEMAPBASE(const TSORTABLEMAPBASE<KeyType, ValueType, OtherSetAllocator, KeyFuncs>& Other)
	: Super(Other)
	{
	}

	/** Assignment operator for moving elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	TSORTABLEMAPBASE& operator=(TSORTABLEMAPBASE<KeyType, ValueType, OtherSetAllocator, KeyFuncs>&& Other)
	{
		(Super&)*this = MoveTemp(Other);
		return *this;
	}

	/** Assignment operator for copying elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	TSORTABLEMAPBASE& operator=(const TSORTABLEMAPBASE<KeyType, ValueType, OtherSetAllocator, KeyFuncs>& Other)
	{
		(Super&)*this = Other;
		return *this;
	}

	/////////////////////////////////////////////////////////
	// Start - intrusive TOptional<TSORTABLEMAPBASE> state //
	/////////////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TSORTABLEMAPBASE;

	[[nodiscard]] explicit TSORTABLEMAPBASE(FIntrusiveUnsetOptionalState Tag)
	: Super(Tag)
	{
	}
	[[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState Tag) const
	{
		return Super::operator==(Tag);
	}
	///////////////////////////////////////////////////////
	// End - intrusive TOptional<TSORTABLEMAPBASE> state //
	///////////////////////////////////////////////////////

public:
	/**
	 * Sorts the pairs array using each pair's Key as the sort criteria, then rebuilds the map's hash.
	 * Invoked using "MyMapVar.KeySort( PREDICATE_CLASS() );"
	 */
	template<typename PREDICATE_CLASS>
	UE_FORCEINLINE_HINT void KeySort(const PREDICATE_CLASS& Predicate)
	{
		Super::Pairs.Sort(FKeyComparisonClass<PREDICATE_CLASS>(Predicate));
	}

	/**
	 * Stable sorts the pairs array using each pair's Key as the sort criteria, then rebuilds the map's hash.
	 * Invoked using "MyMapVar.KeySort( PREDICATE_CLASS() );"
	 */
	template<typename PREDICATE_CLASS>
	UE_FORCEINLINE_HINT void KeyStableSort(const PREDICATE_CLASS& Predicate)
	{
		Super::Pairs.StableSort(FKeyComparisonClass<PREDICATE_CLASS>(Predicate));
	}

	/**
	 * Sorts the pairs array using each pair's Value as the sort criteria, then rebuilds the map's hash.
	 * Invoked using "MyMapVar.ValueSort( PREDICATE_CLASS() );"
	 */
	template<typename PREDICATE_CLASS>
	UE_FORCEINLINE_HINT void ValueSort(const PREDICATE_CLASS& Predicate)
	{
		Super::Pairs.Sort(FValueComparisonClass<PREDICATE_CLASS>(Predicate));
	}

	/**
	 * Stable sorts the pairs array using each pair's Value as the sort criteria, then rebuilds the map's hash.
	 * Invoked using "MyMapVar.ValueSort( PREDICATE_CLASS() );"
	 */
	template<typename PREDICATE_CLASS>
	UE_FORCEINLINE_HINT void ValueStableSort(const PREDICATE_CLASS& Predicate)
	{
		Super::Pairs.StableSort(FValueComparisonClass<PREDICATE_CLASS>(Predicate));
	}

	/**
	 * Sort the free element list so that subsequent additions will occur in the lowest available
	 * TSet index resulting in tighter packing without moving any existing items. Also useful for
	 * some types of determinism. @see TSparseArray::SortFreeList() for more info.
	 */
	void SortFreeList() requires (&Super::ElementSetType::SortFreeList != nullptr)
	{
		Super::Pairs.SortFreeList();
	}

private:

	/** Extracts the pair's key from the map's pair structure and passes it to the user provided comparison class. */
	template<typename PREDICATE_CLASS>
	class FKeyComparisonClass
	{
		TDereferenceWrapper< KeyType, PREDICATE_CLASS> Predicate;

	public:

		[[nodiscard]] UE_FORCEINLINE_HINT FKeyComparisonClass(const PREDICATE_CLASS& InPredicate)
		: Predicate(InPredicate)
		{
		}

		[[nodiscard]] inline bool operator()(const typename Super::ElementType& A, const typename Super::ElementType& B) const
		{
			return Predicate(A.Key, B.Key);
		}
	};

	/** Extracts the pair's value from the map's pair structure and passes it to the user provided comparison class. */
	template<typename PREDICATE_CLASS>
	class FValueComparisonClass
	{
		TDereferenceWrapper< ValueType, PREDICATE_CLASS> Predicate;

	public:

		[[nodiscard]] UE_FORCEINLINE_HINT FValueComparisonClass(const PREDICATE_CLASS& InPredicate)
		: Predicate(InPredicate)
		{
		}

		[[nodiscard]] inline bool operator()(const typename Super::ElementType& A, const typename Super::ElementType& B) const
		{
			return Predicate(A.Value, B.Value);
		}
	};
};

/** A TMAPBASE specialization that only allows a single value associated with each key.*/
template<typename InKeyType, typename InValueType, typename SetAllocator /*= FDefaultSetAllocator*/, typename KeyFuncs /*= TDefaultMapHashableKeyFuncs<KeyType,ValueType,false>*/>
class TMAP : public TSORTABLEMAPBASE<InKeyType, InValueType, SetAllocator, KeyFuncs>
{
	template <typename, typename>
	friend class TScriptMap;

	static_assert(!KeyFuncs::bAllowDuplicateKeys, TMAP_STRINGIFY(TMAP) " cannot be instantiated with a KeyFuncs which allows duplicate keys");

public:
	typedef InKeyType      KeyType;
	typedef InValueType    ValueType;
	typedef SetAllocator   SetAllocatorType;
	typedef KeyFuncs       KeyFuncsType;

	typedef TSORTABLEMAPBASE<KeyType, ValueType, SetAllocator, KeyFuncs> Super;
	typedef typename Super::KeyInitType KeyInitType;

	[[nodiscard]] constexpr TMAP() = default;
	[[nodiscard]] explicit consteval TMAP(EConstEval)
	: Super(ConstEval)
	{
	}
	[[nodiscard]] TMAP(TMAP&&) = default;
	[[nodiscard]] TMAP(const TMAP&) = default;
	TMAP& operator=(TMAP&&) = default;
	TMAP& operator=(const TMAP&) = default;

	/** Constructor for moving elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	[[nodiscard]] TMAP(TMAP<KeyType, ValueType, OtherSetAllocator, KeyFuncs>&& Other)
	: Super(MoveTemp(Other))
	{
	}

	/** Constructor for copying elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	[[nodiscard]] TMAP(const TMAP<KeyType, ValueType, OtherSetAllocator, KeyFuncs>& Other)
	: Super(Other)
	{
	}

	/** Constructor which gets its elements from a native initializer list */
	[[nodiscard]] TMAP(std::initializer_list<TPairInitializer<const KeyType&, const ValueType&>> InitList)
	{
		this->Reserve((int32)InitList.size());
		for (const TPairInitializer<const KeyType&, const ValueType&>& Element : InitList)
		{
			this->Add(Element.Key, Element.Value);
		}
	}

	/////////////////////////////////////////////
	// Start - intrusive TOptional<TMAP> state //
	/////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TMAP;

	[[nodiscard]] explicit TMAP(FIntrusiveUnsetOptionalState Tag)
	: Super(Tag)
	{
	}
	[[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState Tag) const
	{
		return Super::operator==(Tag);
	}
	///////////////////////////////////////////
	// End - intrusive TOptional<TMAP> state //
	///////////////////////////////////////////

	/** Assignment operator for moving elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	TMAP& operator=(TMAP<KeyType, ValueType, OtherSetAllocator, KeyFuncs>&& Other)
	{
		(Super&)*this = MoveTemp(Other);
		return *this;
	}

	/** Assignment operator for copying elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	TMAP& operator=(const TMAP<KeyType, ValueType, OtherSetAllocator, KeyFuncs>& Other)
	{
		(Super&)*this = Other;
		return *this;
	}

	/** Assignment operator which gets its elements from a native initializer list */
	TMAP& operator=(std::initializer_list<TPairInitializer<const KeyType&, const ValueType&>> InitList)
	{
		this->Empty((int32)InitList.size());
		for (const TPairInitializer<const KeyType&, const ValueType&>& Element : InitList)
		{
			this->Add(Element.Key, Element.Value);
		}
		return *this;
	}

	/**
	 * Remove the pair with the specified key and copies the value
	 * that was removed to the ref parameter
	 *
	 * @param Key The key to search for
	 * @param OutRemovedValue If found, the value that was removed (not modified if the key was not found)
	 * @return whether or not the key was found
	 */
	inline bool RemoveAndCopyValue(KeyInitType Key, ValueType& OutRemovedValue)
	{
		const FSetElementId PairId = Super::Pairs.FindId(Key);
		if (!PairId.IsValidId())
		{
			return false;
		}

		OutRemovedValue = MoveTempIfPossible(Super::Pairs[PairId].Value);
		Super::Pairs.Remove(PairId);
		return true;
	}
	
	/** See RemoveAndCopyValue(), set remains compact and in stable order after element is removed */
	inline bool RemoveAndCopyValueStable(KeyInitType Key, ValueType& OutRemovedValue)
	{
		const FSetElementId PairId = Super::Pairs.FindId(Key);
		if (!PairId.IsValidId())
		{
			return false;
		}

		OutRemovedValue = MoveTempIfPossible(Super::Pairs[PairId].Value);
		Super::Pairs.RemoveStable(PairId);
		return true;
	}
	
	/** See RemoveAndCopyValue() and class documentation section on ByHash() functions */
	template<typename ComparableKey>
	inline bool RemoveAndCopyValueByHash(uint32 KeyHash, const ComparableKey& Key, ValueType& OutRemovedValue)
	{
		const FSetElementId PairId = Super::Pairs.FindIdByHash(KeyHash, Key);
		if (!PairId.IsValidId())
		{
			return false;
		}

		OutRemovedValue = MoveTempIfPossible(Super::Pairs[PairId].Value);
		Super::Pairs.Remove(PairId);
		return true;
	}

	/**
	 * Find a pair with the specified key, removes it from the map, and returns the value part of the pair.
	 *
	 * If no pair was found, an exception is thrown.
	 *
	 * @param Key the key to search for
	 * @return whether or not the key was found
	 */
	inline ValueType FindAndRemoveChecked(KeyInitType Key)
	{
		const FSetElementId PairId = Super::Pairs.FindId(Key);
		check(PairId.IsValidId());
		ValueType Result = MoveTempIfPossible(Super::Pairs[PairId].Value);
		Super::Pairs.Remove(PairId);
		return Result;
	}

	/**
	 * Move all items from another map into our map (if any keys are in both,
	 * the value from the other map wins) and empty the other map.
	 *
	 * @param OtherMap The other map of items to move the elements from.
	 */
	template<typename OtherSetAllocator>
	void Append(TMAP<KeyType, ValueType, OtherSetAllocator, KeyFuncs>&& OtherMap)
	{
		this->Reserve(this->Num() + OtherMap.Num());
		for (auto& Pair : OtherMap)
		{
			this->Add(MoveTempIfPossible(Pair.Key), MoveTempIfPossible(Pair.Value));
		}

		OtherMap.Reset();
	}

	/**
	 * Add all items from another map to our map (if any keys are in both,
	 * the value from the other map wins).
	 *
	 * @param OtherMap The other map of items to add.
	 */
	template<typename OtherSetAllocator>
	void Append(const TMAP<KeyType, ValueType, OtherSetAllocator, KeyFuncs>& OtherMap)
	{
		this->Reserve(this->Num() + OtherMap.Num());
		for (auto& Pair : OtherMap)
		{
			this->Add(Pair.Key, Pair.Value);
		}
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ValueType& operator[](KeyInitType Key)
	{
		return this->FindChecked(Key);
	}
	[[nodiscard]] UE_FORCEINLINE_HINT const ValueType& operator[](KeyInitType Key) const
	{
		return this->FindChecked(Key);
	}
};

namespace Freeze
{
	template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
	void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const TMAP<KeyType, ValueType, SetAllocator, KeyFuncs>& Object, const FTypeLayoutDesc&)
	{
		Object.WriteMemoryImage(Writer);
	}

	template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
	uint32 IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const TMAP<KeyType, ValueType, SetAllocator, KeyFuncs>& Object, void* OutDst)
	{
		Object.CopyUnfrozen(Context, OutDst);
		return sizeof(Object);
	}

	template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
	uint32 IntrinsicAppendHash(const TMAP<KeyType, ValueType, SetAllocator, KeyFuncs>* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		TMAP<KeyType, ValueType, SetAllocator, KeyFuncs>::AppendHash(LayoutParams, Hasher);
		return DefaultAppendHash(TypeDesc, LayoutParams, Hasher);
	}
}

DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT((template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>), (TMAP<KeyType, ValueType, SetAllocator, KeyFuncs>));

/** A TMAPBASE specialization that allows multiple values to be associated with each key. */
template<typename KeyType, typename ValueType, typename SetAllocator /* = FDefaultSetAllocator */, typename KeyFuncs /*= TDefaultMapHashableKeyFuncs<KeyType,ValueType,true>*/ /*= FDefaultSetAllocator*/>
class TMULTIMAP : public TSORTABLEMAPBASE<KeyType, ValueType, SetAllocator, KeyFuncs>
{
	static_assert(KeyFuncs::bAllowDuplicateKeys, TMAP_STRINGIFY(TMULTIMAP) " cannot be instantiated with a KeyFuncs which disallows duplicate keys");

public:
	typedef TSORTABLEMAPBASE<KeyType, ValueType, SetAllocator, KeyFuncs> Super;
	typedef typename Super::KeyInitType KeyInitType;
	typedef typename Super::ValueInitType ValueInitType;

	[[nodiscard]] constexpr TMULTIMAP() = default;
	[[nodiscard]] TMULTIMAP(TMULTIMAP&&) = default;
	[[nodiscard]] TMULTIMAP(const TMULTIMAP&) = default;
	TMULTIMAP& operator=(TMULTIMAP&&) = default;
	TMULTIMAP& operator=(const TMULTIMAP&) = default;

	/** Constructor for moving elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	[[nodiscard]] TMULTIMAP(TMULTIMAP<KeyType, ValueType, OtherSetAllocator, KeyFuncs>&& Other)
	: Super(MoveTemp(Other))
	{
	}

	/** Constructor for copying elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	[[nodiscard]] TMULTIMAP(const TMULTIMAP<KeyType, ValueType, OtherSetAllocator, KeyFuncs>& Other)
	: Super(Other)
	{
	}

	/** Constructor which gets its elements from a native initializer list */
	[[nodiscard]] TMULTIMAP(std::initializer_list<TPairInitializer<const KeyType&, const ValueType&>> InitList)
	{
		this->Reserve((int32)InitList.size());
		for (const TPairInitializer<const KeyType&, const ValueType&>& Element : InitList)
		{
			this->Add(Element.Key, Element.Value);
		}
	}

	//////////////////////////////////////////////////
	// Start - intrusive TOptional<TMULTIMAP> state //
	//////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TMULTIMAP;

	[[nodiscard]] explicit TMULTIMAP(FIntrusiveUnsetOptionalState Tag)
	: Super(Tag)
	{
	}
	[[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState Tag) const
	{
		return Super::operator==(Tag);
	}
	////////////////////////////////////////////////
	// End - intrusive TOptional<TMULTIMAP> state //
	////////////////////////////////////////////////

	/** Assignment operator for moving elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	TMULTIMAP& operator=(TMULTIMAP<KeyType, ValueType, OtherSetAllocator, KeyFuncs>&& Other)
	{
		(Super&)*this = MoveTemp(Other);
		return *this;
	}

	/** Assignment operator for copying elements from a TMAP with a different SetAllocator */
	template<typename OtherSetAllocator>
	TMULTIMAP& operator=(const TMULTIMAP<KeyType, ValueType, OtherSetAllocator, KeyFuncs>& Other)
	{
		(Super&)*this = Other;
		return *this;
	}

	/** Assignment operator which gets its elements from a native initializer list */
	TMULTIMAP& operator=(std::initializer_list<TPairInitializer<const KeyType&, const ValueType&>> InitList)
	{
		this->Empty((int32)InitList.size());
		for (const TPairInitializer<const KeyType&, const ValueType&>& Element : InitList)
		{
			this->Add(Element.Key, Element.Value);
		}
		return *this;
	}

	/**
	 * Finds all values associated with the specified key.
	 *
	 * @param Key The key to find associated values for.
	 * @param OutValues Upon return, contains the values associated with the key.
	 * @param bMaintainOrder true if the Values array should be in the same order as the map's pairs.
	 */
	template<typename Allocator> void MultiFind(KeyInitType Key, TArray<ValueType, Allocator>& OutValues, bool bMaintainOrder = false) const
	{
		for (typename Super::ElementSetType::TConstKeyIterator It(Super::Pairs, Key); It; ++It)
		{
			OutValues.Add(It->Value);
		}

		if (bMaintainOrder)
		{
			Algo::Reverse(OutValues);
		}
	}

	/**
	 * Finds all values associated with the specified key.
	 *
	 * @param Key The key to find associated values for.
	 * @param OutValues Upon return, contains pointers to the values associated with the key.
	 *					Pointers are only valid until the next change to any key in the map.
	 * @param bMaintainOrder true if the Values array should be in the same order as the map's pairs.
	 */
	template<typename Allocator> void MultiFindPointer(KeyInitType Key, TArray<const ValueType*, Allocator>& OutValues, bool bMaintainOrder = false) const
	{
		for (typename Super::ElementSetType::TConstKeyIterator It(Super::Pairs, Key); It; ++It)
		{
			OutValues.Add(&It->Value);
		}

		if (bMaintainOrder)
		{
			Algo::Reverse(OutValues);
		}
	}
	template<typename Allocator> void MultiFindPointer(KeyInitType Key, TArray<ValueType*, Allocator>& OutValues, bool bMaintainOrder = false)
	{
		for (typename Super::ElementSetType::TKeyIterator It(Super::Pairs, Key); It; ++It)
		{
			OutValues.Add(&It->Value);
		}

		if (bMaintainOrder)
		{
			Algo::Reverse(OutValues);
		}
	}

	/**
	 * Add a key-value association to the map.  The association doesn't replace any of the key's existing associations.
	 * However, if both the key and value match an existing association in the map, no new association is made and the existing association's
	 * value is returned.
	 *
	 * @param InKey The key to associate.
	 * @param InValue The value to associate.
	 * @return A reference to the value as stored in the map; the reference is only valid until the next change to any key in the map.
	 */
	UE_FORCEINLINE_HINT ValueType& AddUnique(const KeyType&  InKey, const ValueType&  InValue) { return EmplaceUnique(InKey, InValue); }
	UE_FORCEINLINE_HINT ValueType& AddUnique(const KeyType&  InKey, ValueType&& InValue) { return EmplaceUnique(InKey, MoveTempIfPossible(InValue)); }
	UE_FORCEINLINE_HINT ValueType& AddUnique(KeyType&& InKey, const ValueType&  InValue) { return EmplaceUnique(MoveTempIfPossible(InKey), InValue); }
	UE_FORCEINLINE_HINT ValueType& AddUnique(KeyType&& InKey, ValueType&& InValue) { return EmplaceUnique(MoveTempIfPossible(InKey), MoveTempIfPossible(InValue)); }

	/**
	 * Add a key-value association to the map.
	 *
	 * The association doesn't replace any of the key's existing associations.
	 * However, if both key and value match an existing association in the map,
	 * no new association is made and the existing association's value is returned.
	 *
	 * @param InKey The key to associate.
	 * @param InValue The value to associate.
	 * @return A reference to the value as stored in the map; the reference is only valid until the next change to any key in the map.
	 */
	template <typename InitKeyType, typename InitValueType>
	ValueType& EmplaceUnique(InitKeyType&& InKey, InitValueType&& InValue)
	{
		if (ValueType* Found = FindPair(InKey, InValue))
		{
			return *Found;
		}

		// If there's no existing association with the same key and value, create one.
		return Super::Add(Forward<InitKeyType>(InKey), Forward<InitValueType>(InValue));
	}

	/**
	 * Remove all value associations for a key.
	 *
	 * @param InKey The key to remove associated values for.
	 * @return The number of values that were associated with the key.
	 */
	UE_FORCEINLINE_HINT int32 Remove(KeyInitType InKey)
	{
		return Super::Remove(InKey);
	}

	/**
	 * Remove associations between the specified key and value from the map.
	 *
	 * @param InKey The key part of the pair to remove.
	 * @param InValue The value part of the pair to remove.
	 * @return The number of associations removed.
	 */
	int32 Remove(KeyInitType InKey, ValueInitType InValue)
	{
		// Iterate over pairs with a matching key.
		int32 NumRemovedPairs = 0;
		for (typename Super::ElementSetType::TKeyIterator It(Super::Pairs, InKey); It; ++It)
		{
			// If this pair has a matching value as well, remove it.
			if (It->Value == InValue)
			{
				It.RemoveCurrent();
				++NumRemovedPairs;
			}
		}
		return NumRemovedPairs;
	}

	/**
	 * Remove the first association between the specified key and value from the map.
	 *
	 * @param InKey The key part of the pair to remove.
	 * @param InValue The value part of the pair to remove.
	 * @return The number of associations removed.
	 */
	int32 RemoveSingle(KeyInitType InKey, ValueInitType InValue)
	{
		// Iterate over pairs with a matching key.
		int32 NumRemovedPairs = 0;
		for (typename Super::ElementSetType::TKeyIterator It(Super::Pairs, InKey); It; ++It)
		{
			// If this pair has a matching value as well, remove it.
			if (It->Value == InValue)
			{
				It.RemoveCurrent();
				++NumRemovedPairs;

				// We were asked to remove only the first association, so bail out.
				break;
			}
		}
		return NumRemovedPairs;
	}

	/**
	 * Remove the first association between the specified key and value from the map. Ensure the map order remains unchanged
	 *
	 * @param InKey The key part of the pair to remove.
	 * @param InValue The value part of the pair to remove.
	 * @return The number of associations removed.
	 */
	bool RemoveSingleStable(KeyInitType InKey, ValueInitType InValue)
	{
		for (typename Super::ElementSetType::TKeyIterator It(Super::Pairs, InKey); It; ++It)
		{
			if (It->Value == InValue)
			{
				Super::Pairs.RemoveStable(It.GetId());
				return true;
			}
		}
		return false;
	}

	/**
	 * Find an association between a specified key and value. (const)
	 *
	 * @param Key The key to find.
	 * @param Value The value to find.
	 * @return If the map contains a matching association, a pointer to the value in the map is returned.  Otherwise nullptr is returned.
	 *			The pointer is only valid as long as the map isn't changed.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const ValueType* FindPair(KeyInitType Key, ValueInitType Value) const
	{
		return const_cast<TMULTIMAP*>(this)->FindPair(Key, Value);
	}

	/**
	 * Find an association between a specified key and value.
	 *
	 * @param Key The key to find.
	 * @param Value The value to find.
	 * @return If the map contains a matching association, a pointer to the value in the map is returned.  Otherwise nullptr is returned.
	 *			The pointer is only valid as long as the map isn't changed.
	 */
	[[nodiscard]] ValueType* FindPair(KeyInitType Key, ValueInitType Value)
	{
		// Iterate over pairs with a matching key.
		for (typename Super::ElementSetType::TKeyIterator It(Super::Pairs, Key); It; ++It)
		{
			// If the pair's value matches, return a pointer to it.
			if (It->Value == Value)
			{
				return &It->Value;
			}
		}

		return nullptr;
	}

	/** Returns the number of values within this map associated with the specified key */
	[[nodiscard]] int32 Num(KeyInitType Key) const
	{
		// Iterate over pairs with a matching key.
		int32 NumMatchingPairs = 0;
		for (typename Super::ElementSetType::TConstKeyIterator It(Super::Pairs, Key); It; ++It)
		{
			++NumMatchingPairs;
		}
		return NumMatchingPairs;
	}

	// Since we implement an overloaded Num() function in TMULTIMAP, we need to reimplement TMAPBASE::Num to make it visible.
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Num() const
	{
		return Super::Num();
	}

	/**
	 * Move all items from another map into our map (if any keys are in both,
	 * the value from the other map wins) and empty the other map.
	 *
	 * @param OtherMultiMap The other map of items to move the elements from.
	 */
	template<typename OtherSetAllocator>
	void Append(TMULTIMAP<KeyType, ValueType, OtherSetAllocator, KeyFuncs>&& OtherMultiMap)
	{
		this->Reserve(this->Num() + OtherMultiMap.Num());
		for (auto& Pair : OtherMultiMap)
		{
			this->Add(MoveTempIfPossible(Pair.Key), MoveTempIfPossible(Pair.Value));
		}

		OtherMultiMap.Reset();
	}

	/**
	 * Add all items from another map to our map (if any keys are in both,
	 * the value from the other map wins).
	 *
	 * @param OtherMultiMap The other map of items to add.
	 */
	template<typename OtherSetAllocator>
	void Append(const TMULTIMAP<KeyType, ValueType, OtherSetAllocator, KeyFuncs>& OtherMultiMap)
	{
		this->Reserve(this->Num() + OtherMultiMap.Num());
		for (auto& Pair : OtherMultiMap)
		{
			this->Add(Pair.Key, Pair.Value);
		}
	}
};

namespace Freeze
{
	template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
	void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const TMULTIMAP<KeyType, ValueType, SetAllocator, KeyFuncs>& Object, const FTypeLayoutDesc&)
	{
		Object.WriteMemoryImage(Writer);
	}

	template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
	uint32 IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const TMULTIMAP<KeyType, ValueType, SetAllocator, KeyFuncs>& Object, void* OutDst)
	{
		Object.CopyUnfrozen(Context, OutDst);
		return sizeof(Object);
	}

	template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
	uint32 IntrinsicAppendHash(const TMULTIMAP<KeyType, ValueType, SetAllocator, KeyFuncs>* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		TMULTIMAP<KeyType, ValueType, SetAllocator, KeyFuncs>::AppendHash(LayoutParams, Hasher);
		return DefaultAppendHash(TypeDesc, LayoutParams, Hasher);
	}
}

DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT((template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>), (TMULTIMAP<KeyType, ValueType, SetAllocator, KeyFuncs>));

template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
inline constexpr bool TIsTMap_V<TMAP<KeyType, ValueType, SetAllocator, KeyFuncs>> = true;

struct TMAPPRIVATEFRIEND
{
	template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
	inline static FArchive& Serialize(FArchive& Ar, TMAPBASE<KeyType, ValueType, SetAllocator, KeyFuncs>& Map)
	{
		Ar << Map.Pairs;
		return Ar;
	}

	template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
	UE_FORCEINLINE_HINT static void SerializeStructured(FStructuredArchive::FSlot Slot, TMAPBASE<KeyType, ValueType, SetAllocator, KeyFuncs>& InMap)
	{
		/*
		if (Slot.GetUnderlyingArchive().IsTextFormat())
		{
			int32 Num = InMap.Num();
			FStructuredArchive::FMap Map = Slot.EnterMap(Num);

			if (Slot.GetUnderlyingArchive().IsLoading())
			{
				FString KeyString;
				KeyType Key;

				for (int32 Index = 0; Index < Num; ++Index)
				{
					FStructuredArchive::FSlot ValueSlot = Map.EnterElement(KeyString);
					LexFromString(Key, *KeyString);
					ValueSlot << InMap.Add(Key);
				}
			}
			else
			{
				FString StringK;
				for (TMAPBASE::TIterator It(InMap); It; ++It)
				{
					StringK = LexToString(It->Key);
					FStructuredArchive::FSlot ValueSlot = Map.EnterElement(StringK);
					ValueSlot << It->Value;
				}
			}
		}
		else
		*/
		{
			Slot << InMap.Pairs;
		}
	}

	template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
	[[nodiscard]] static bool LegacyCompareEqual(const TMAPBASE<KeyType, ValueType, SetAllocator, KeyFuncs>& A, const TMAPBASE<KeyType, ValueType, SetAllocator, KeyFuncs>& B)
	{
		return TSetPrivateFriend::LegacyCompareEqual(A.Pairs, B.Pairs);
	}
};

/** Serializer. */
template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
UE_FORCEINLINE_HINT FArchive& operator<<(FArchive& Ar, TMAPBASE<KeyType, ValueType, SetAllocator, KeyFuncs>& Map)
{
	return TMAPPRIVATEFRIEND::Serialize(Ar, Map);
}

/** Structured archive serializer. */
template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
UE_FORCEINLINE_HINT void operator<<(FStructuredArchive::FSlot Slot, TMAPBASE<KeyType, ValueType, SetAllocator, KeyFuncs>& InMap)
{
	TMAPPRIVATEFRIEND::SerializeStructured(Slot, InMap);
}

// Legacy comparison operators.  Note that these also test whether the map's key-value pairs were added in the same order!
template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
[[nodiscard]] bool LegacyCompareEqual(const TMAPBASE<KeyType, ValueType, SetAllocator, KeyFuncs>& A, const TMAPBASE<KeyType, ValueType, SetAllocator, KeyFuncs>& B)
{
	return TMAPPRIVATEFRIEND::LegacyCompareEqual(A, B);
}
template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
[[nodiscard]] bool LegacyCompareNotEqual(const TMAPBASE<KeyType, ValueType, SetAllocator, KeyFuncs>& A, const TMAPBASE<KeyType, ValueType, SetAllocator, KeyFuncs>& B)
{
	return !TMAPPRIVATEFRIEND::LegacyCompareEqual(A, B);
}
