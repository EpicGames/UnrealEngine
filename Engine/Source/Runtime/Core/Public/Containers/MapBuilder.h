// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"

/**
 * Template for fluent map builders.
 *
 * @param KeyType The type of keys stored in the map.
 * @param ValueType The type of values stored in the map.
 * @param SetAllocator The allocator to use for key-value pairs.
 */
template<typename KeyType, typename ValueType, typename SetAllocator = FDefaultSetAllocator> class TMapBuilder
{
public:

	/** Default constructor. */
	[[nodiscard]] TMapBuilder()
	{
	}

	/**
	 * Creates and initializes a new map builder from another map.
	 *
	 * @param InMap The map to copy.
	 */
	template<typename OtherAllocator>
	[[nodiscard]] TMapBuilder( const TMap<KeyType, ValueType, OtherAllocator>& InMap )
		: Map(InMap)
	{
	}

public:

	/**
	 * Adds a key-value pair to the map.
	 *
	 * @param InKey The key of the pair to add.
	 * @param InValue The value of the pair to add.
	 * @return This instance (for method chaining).
	 */
	TMapBuilder& Add( KeyType InKey, ValueType InValue )
	{
		Map.Add(InKey, InValue);

		return *this;
	}

	/**
	 * Appends another map.
	 *
	 * @param OtherMap The map to append.
	 * @return This instance (for method chaining).
	 */
	TMapBuilder& Append( const TMap<KeyType, ValueType, SetAllocator>& OtherMap )
	{
		Map.Append(OtherMap);

		return *this;
	}

public:

	/**
	 * Builds the map as configured.
	 *
	 * @return A new map.
	 */
	[[nodiscard]] TMap<KeyType, ValueType, SetAllocator> Build()
	{
		return Map;
	}

	/**
	 * Implicit conversion operator to build the map as configured.
	 *
	 * @return A new map.
	 */
	[[nodiscard]] operator TMap<KeyType, ValueType, SetAllocator>()
	{
		return Build();
	}

private:

	/** Holds the map being built. */
	TMap<KeyType, ValueType, SetAllocator> Map;
};
