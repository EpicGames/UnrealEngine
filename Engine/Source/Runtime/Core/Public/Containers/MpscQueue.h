// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "AutoRTFM/ReportHazard.h"
#include "Templates/MemoryOps.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/UnrealTemplate.h"
#include "Misc/Optional.h"
#include <atomic>

/** 
 * Fast multi-producer/single-consumer unbounded concurrent queue.
 * Based on http://www.1024cores.net/home/lock-free-algorithms/queues/non-intrusive-mpsc-node-based-queue
 */
template<typename T, typename AllocatorType = FMemory>
class TMpscQueue final
{
public:
	using ElementType = T;

	UE_NONCOPYABLE(TMpscQueue);

	[[nodiscard]] TMpscQueue() : TMpscQueue(::new(AllocatorType::Malloc(sizeof(FNode), alignof(FNode))) FNode)
	{
	}

	~TMpscQueue()
	{
		FNode* Next = Tail->Next.load(std::memory_order_relaxed);

		// sentinel's value is already destroyed
		AllocatorType::Free(Tail);

		while (Next != nullptr)
		{
			Tail = Next;
			Next = Tail->Next.load(std::memory_order_relaxed);

			DestructItem((ElementType*)&Tail->Value);
			AllocatorType::Free(Tail);
		}
	}

	template <typename... ArgTypes>
	void Enqueue(ArgTypes&&... Args)
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(MpscQueue);

		FNode* New = ::new(AllocatorType::Malloc(sizeof(FNode), alignof(FNode))) FNode;
		::new ((void*)&New->Value) ElementType(Forward<ArgTypes>(Args)...);

		FNode* Prev = Head.exchange(New, std::memory_order_acq_rel);
		Prev->Next.store(New, std::memory_order_release);
	}

	TOptional<ElementType> Dequeue()
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(MpscQueue);

		FNode* Next = Tail->Next.load(std::memory_order_acquire);

		if (Next == nullptr)
		{
			return {};
		}

		ElementType* ValuePtr = (ElementType*)&Next->Value;
		TOptional<ElementType> Res{ MoveTemp(*ValuePtr) };
		DestructItem(ValuePtr);

		AllocatorType::Free(Tail); // current sentinel

		Tail = Next; // new sentinel
		return Res;
	}

	bool Dequeue(ElementType& OutElem)
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(MpscQueue);

		TOptional<ElementType> LocalElement = Dequeue();
		if (LocalElement.IsSet())
		{
			OutElem = MoveTempIfPossible(LocalElement.GetValue());
			return true;
		}

		return false;
	}

	// as there can be only one consumer, a consumer can safely "peek" the tail of the queue.
	// returns a pointer to the tail if the queue is not empty, nullptr otherwise
	// there's no overload with TOptional as it doesn't support references
	[[nodiscard]] ElementType* Peek() const
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(MpscQueue);

		FNode* Next = Tail->Next.load(std::memory_order_acquire);

		if (Next == nullptr)
		{
			return nullptr;
		}

		return (ElementType*)&Next->Value;
	}

	[[nodiscard]] bool IsEmpty() const
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(MpscQueue);

		return Tail->Next.load(std::memory_order_acquire) == nullptr;
	}

private:
	struct FNode
	{
		std::atomic<FNode*> Next{ nullptr };
		TTypeCompatibleBytes<ElementType> Value;
	};

	[[nodiscard]] explicit TMpscQueue(FNode* Sentinel) : Head(Sentinel), Tail(Sentinel)
	{
	}

private:
	std::atomic<FNode*> Head; // accessed only by producers
	/*alignas(PLATFORM_CACHE_LINE_SIZE) */FNode* Tail; // accessed only by consumer, hence should be on a different cache line than `Head`
};
