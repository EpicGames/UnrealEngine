// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "IteratorAdapter.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/UnrealTemplate.h"  // For GetData, GetNum
#include "Traits/IsTriviallyRelocatable.h"

#include <cstring>
#include <initializer_list>
#include <type_traits>

template <typename InElementType, int32 InPageSizeInBytes = 16384, typename InAllocatorType = FDefaultAllocator>
class TPagedArray;

namespace UE::Core::PagedArray::Private
{

/**
 * Page traits struct for TPagedArray.
 */
template <typename InElementType, int32 InPageSizeInBytes>
struct TPageTraits
{
	static_assert(InPageSizeInBytes >= sizeof(InElementType), "Page size must be greater or equal than element type's");

	using ElementType = InElementType;

	static constexpr int32 Size = InPageSizeInBytes;
	static constexpr int32 Capacity = Size / sizeof(ElementType);
};

/**
 * TPagedArray iterator base class for TIteratorAdapter.
 */
template <typename InElementType, typename InPageType, typename InPageTraits>
class TIteratorBase
{
public:
	using ElementType = InElementType;
	using PageType = InPageType;
	using PageTraits = InPageTraits;
	using SizeType = typename PageType::SizeType;

	TIteratorBase() = default;

protected:
	/**
	 * Internal parameter constructor (used exclusively by the container).
	 */
	[[nodiscard]] TIteratorBase(InPageType* Data, SizeType Offset)
		: Data(Data), PageIndex(Offset / PageTraits::Capacity), PageOffset(Offset % PageTraits::Capacity)
	{
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& Dereference() const
	{
		return Data[PageIndex][PageOffset];
	}

	inline void Increment()
	{
		if (++PageOffset == PageTraits::Capacity)
		{
			PageOffset = 0;
			++PageIndex;
		}
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool Equals(const TIteratorBase& Other) const
	{
		return Data == Other.Data && PageIndex == Other.PageIndex && PageOffset == Other.PageOffset;
	}

private:
	PageType* Data = nullptr;
	SizeType PageIndex = 0;
	SizeType PageOffset = 0;
};

template <typename InElementType, typename InPageType, typename InPageTraits>
using TIterator = TIteratorAdapter<TIteratorBase<InElementType, InPageType, InPageTraits>>;

/** Serializer. */
template <typename ElementType, int32 PageSizeInBytes, typename AllocatorType>
FArchive& Serialize(FArchive& Ar, TPagedArray<ElementType, PageSizeInBytes, AllocatorType>& InOutPagedArray)
{
	using SizeType = typename AllocatorType::SizeType;

	InOutPagedArray.CountBytes(Ar);

	SizeType NumElements = InOutPagedArray.Num();
	Ar << NumElements;

	if (Ar.IsLoading())
	{
		InOutPagedArray.Empty(NumElements);

		for (SizeType ElementIndex = 0; ElementIndex < NumElements; ++ElementIndex)
		{
			Ar << InOutPagedArray.Emplace_GetRef();
		}
	}
	else
	{
		for (ElementType& Element : InOutPagedArray)
		{
			Ar << Element;
		}
	}
	return Ar;
}

/** Structured archive serializer. */
template <typename ElementType, int32 PageSizeInBytes, typename AllocatorType>
void SerializeStructured(FStructuredArchive::FSlot Slot, TPagedArray<ElementType, PageSizeInBytes, AllocatorType>& InOutPagedArray)
{
	int32 NumElements = InOutPagedArray.Num();
	FStructuredArchive::FArray Array = Slot.EnterArray(NumElements);
	if (Slot.GetUnderlyingArchive().IsLoading())
	{
		InOutPagedArray.Empty(NumElements);

		for (int32 ElementIndex = 0; ElementIndex < NumElements; ++ElementIndex)
		{
			FStructuredArchive::FSlot ElementSlot = Array.EnterElement();
			ElementSlot << InOutPagedArray.Emplace_GetRef();
		}
	}
	else
	{
		for (ElementType& Element : InOutPagedArray)
		{
			FStructuredArchive::FSlot ElementSlot = Array.EnterElement();
			ElementSlot << Element;
		}
	}
}

}  // namespace UE::Core::PagedArray::Private

/**
 * Fixed size block allocated container class.
 *
 * This container mimics the behavior of an array without fulfilling the contiguous storage guarantee. Instead, it
 * allocates memory in pages. Each page is a block of memory of the parameter page size in bytes. Elements within a page
 * are guaranteed to be contiguous. Paged arrays have the following advantages v normal arrays:
 * 1. It's more suited for large sized arrays as they don't require a single contiguous memory block. This make them
 *    less vulnerable to memory fragmentation.
 * 2. The contained elements' addresses are persistent as long as the container isn't resized.
 **/
template <typename InElementType, int32 InPageSizeInBytes, typename InAllocatorType>
class TPagedArray
{
	template <typename AnyElementType, int32 AnyPageSizeInBytes, typename AnyAllocatorType>
	friend class TPagedArray;

	using PageType = TArray<InElementType, InAllocatorType>;
	using PageTraits = UE::Core::PagedArray::Private::TPageTraits<InElementType, InPageSizeInBytes>;

public:
	using AllocatorType = InAllocatorType;
	using SizeType = typename InAllocatorType::SizeType;
	using ElementType = InElementType;
	using ElementAllocatorType = std::conditional_t<
		AllocatorType::NeedsElementType,
		typename AllocatorType::template ForElementType<ElementType>,
		typename AllocatorType::ForAnyElementType>;

private:
	[[nodiscard]] static constexpr SizeType GetPageIndex(SizeType Index)
	{
		return Index / PageTraits::Capacity;
	}

	[[nodiscard]] static constexpr SizeType GetPageOffset(SizeType Index)
	{
		return Index % PageTraits::Capacity;
	}

	[[nodiscard]] static constexpr SizeType NumRequiredPages(SizeType Count)
	{
		const SizeType Div = Count / PageTraits::Capacity;
		return Count % PageTraits::Capacity == 0 ? Div : Div + 1;
	}

public:
	using ConstIteratorType = UE::Core::PagedArray::Private::TIterator<const ElementType, const PageType, PageTraits>;
	using IteratorType = UE::Core::PagedArray::Private::TIterator<ElementType, PageType, PageTraits>;

	[[nodiscard]] static constexpr SizeType MaxPerPage()
	{
		return PageTraits::Capacity;
	}

	[[nodiscard]] TPagedArray() = default;

	[[nodiscard]] TPagedArray(TPagedArray&& Other) : Pages(MoveTemp(Other.Pages)), Count(Other.Count)
	{
		Other.Count = 0;
	}

	[[nodiscard]] TPagedArray(const TPagedArray& Other) = default;

	[[nodiscard]] TPagedArray(std::initializer_list<ElementType> InList)
	{
		CopyToEmpty(InList.begin(), static_cast<SizeType>(InList.size()));
	}

	[[nodiscard]] TPagedArray(const ElementType* InSource, SizeType InSize)
	{
		check(InSource || !InSize);
		CopyToEmpty(InSource, InSize);
	}

	~TPagedArray()
	{
		UE_STATIC_ASSERT_WARN(TIsTriviallyRelocatable_V<InElementType>, "TArray can only be used with trivially relocatable types");
	}

	TPagedArray& operator=(TPagedArray&& Other)
	{
		if (this != &Other)
		{
			Pages = MoveTemp(Other.Pages);
			Count = Other.Count;
			Other.Count = 0;
		}
		return *this;
	}

	TPagedArray& operator=(const TPagedArray& Other) = default;

	TPagedArray& operator=(std::initializer_list<ElementType> InList)
	{
		Assign(InList);
		return *this;
	}

	[[nodiscard]] inline const ElementType& operator[](SizeType Index) const
	{
		CheckValidIndex(Index);
		return Pages[GetPageIndex(Index)][GetPageOffset(Index)];
	}

	[[nodiscard]] inline ElementType& operator[](SizeType Index)
	{
		CheckValidIndex(Index);
		return Pages[GetPageIndex(Index)][GetPageOffset(Index)];
	}

	/**
	 * Helper function to return the amount of memory allocated by this container.
	 * Only returns the size of allocations made directly by the container, not the elements themselves.
	 *
	 * @returns Number of bytes allocated by this container.
	 */
	[[nodiscard]] inline SIZE_T GetAllocatedSize() const
	{
		SIZE_T Size = 0;
		Size += Pages.GetAllocatedSize();
		for (const PageType& Page : Pages)
		{
			Size += Page.GetAllocatedSize();
		}
		return Size;
	}

	/**
	 * Count bytes needed to serialize this paged array.
	 *
	 * @param Ar Archive to count for.
	 */
	void CountBytes(FArchive& Ar) const
	{
		Ar.CountBytes(Num() * sizeof(ElementType), Max() * sizeof(ElementType));
	}

	/**
	 * Returns maximum number of elements in array before reallocation.
	 *
	 * @returns Maximum number of elements in array before reallocation.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT SizeType Max() const
	{
		return Pages.Num() * PageTraits::Capacity;
	}

	/**
	 * Returns number of elements in array.
	 *
	 * @returns Number of elements in array.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT SizeType Num() const
	{
		return Count;
	}

	/**
	 * Returns maximum number of pages before reallocation.
	 *
	 * @returns Returns maximum number of pages before reallocation.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT SizeType MaxPages() const
	{
		return Pages.Num();
	}

	/**
	 * Returns number of pages used for the number of elements in array.
	 *
	 * @returns Returns number of pages used for the number of elements in array.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT SizeType NumPages() const
	{
		SizeType NumPages = NumRequiredPages(Count);
		checkSlow(static_cast<int32>(NumPages) <= Pages.Num());
		return NumPages;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValidIndex(SizeType Index) const
	{
		return Index >= 0 && Index < Count;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return !Count;
	}

	[[nodiscard]] inline const ElementType& Last() const
	{
		CheckValidIndex(0);
		const SizeType LastIndex = Num() - 1;
		return Pages[GetPageIndex(LastIndex)][GetPageOffset(LastIndex)];
	}

	[[nodiscard]] inline ElementType& Last()
	{
		CheckValidIndex(0);
		const SizeType LastIndex = Num() - 1;
		return Pages[GetPageIndex(LastIndex)][GetPageOffset(LastIndex)];
	}

	/**
	 * Resizes array to the parameter number of elements.
	 * The allow shrinking parameter indicates whether the container page allocation can be reduced if possible.
	 */
	void SetNum(SizeType NewNum, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		const SizeType RequiredPageCount = NumRequiredPages(NewNum);
		if (NewNum > Num())
		{
			Pages.SetNum(RequiredPageCount);
			SizeType PageIndex = 0;
			SizeType PendingCount = NewNum;
			while (PendingCount >= PageTraits::Capacity)
			{
				Pages[PageIndex++].SetNum(PageTraits::Capacity);
				PendingCount -= PageTraits::Capacity;
			}
			if (PendingCount)
			{
				Pages[PageIndex].Reserve(PageTraits::Capacity);
				Pages[PageIndex].SetNum(PendingCount, EAllowShrinking::No);
			}
		}
		else if (NewNum < Num())
		{
			SizeType PendingCount = Num() - NewNum;
			Pages.SetNum(RequiredPageCount, AllowShrinking);
			if (const SizeType Mod = NewNum % PageTraits::Capacity)
			{
				Pages.Last().SetNum(Mod, EAllowShrinking::No);
			}
		}
		Count = NewNum;
	}

	/*
	 * Fills the memory used by the container elements with the parameter byte value.
	 */
	void SetMem(uint8 ByteValue)
	{
		for (PageType& Page : Pages)
		{
			if (Page.IsEmpty())
			{
				break;
			}
			std::memset((void*)Page.GetData(), ByteValue, Page.Num() * sizeof(ElementType));
		}
	}

	/*
	 * Sets the memory used by the container elements to zero.
	 */
	UE_FORCEINLINE_HINT void SetZero()
	{
		SetMem(0);
	}

	/*
	 * Appends the parameter contiguous range to this container.
	 */
	void Append(const ElementType* InSource, SizeType InSize)
	{
		check(InSource || !InSize);
		GrowIfRequired(Count + InSize);
		CopyUnchecked(InSource, InSize);
	}

	void Append(std::initializer_list<ElementType> InList)
	{
		Append(InList.begin(), static_cast<SizeType>(InList.size()));
	}

	template <
		typename ContainerType
		UE_REQUIRES(TIsContiguousContainer<ContainerType>::Value)
	>
	UE_FORCEINLINE_HINT void Append(ContainerType&& Container)
	{
		Append(GetData(Container), GetNum(Container));
	}

	/*
	 * Appends a compatible paged array to this container.
	 * Note TPagedArray doesn't meet TIsContiguousContainer traits so it won't use the contiguous container overload.
	 */
	template <int32 OtherPageSizeInBytes, typename OtherAllocator>
	void Append(const TPagedArray<ElementType, OtherPageSizeInBytes, OtherAllocator>& Other)
	{
		using OtherType = TPagedArray<ElementType, OtherPageSizeInBytes, OtherAllocator>;
		using OtherPageType = typename OtherType::PageType;
		GrowIfRequired(Count + Other.Count);
		for (const OtherPageType& Page : Other.Pages)
		{
			CopyUnchecked(Page.GetData(), Page.Num());
		}
	}

	/**
	 * Assigns the parameter contiguous range to this container.
	 * Any pre-existing content is removed.
	 */
	void Assign(const ElementType* InSource, SizeType InSize)
	{
		Reset();
		Append(InSource, InSize);
	}

	void Assign(std::initializer_list<ElementType> InList)
	{
		Reset();
		Append(InList.begin(), static_cast<SizeType>(InList.size()));
	}

	template <
		typename ContainerType
		UE_REQUIRES(TIsContiguousContainer<ContainerType>::Value)
	>
	void Assign(ContainerType&& Container)
	{
		Reset();
		Append(GetData(Container), GetNum(Container));
	}

	/**
	 * Assigns a compatible paged array to this container.
	 * Any pre-existing content is removed.
	 * Note TPagedArray doesn't meet TIsContiguousContainer traits so it won't use the contiguous container overload.
	 */
	template <int32 OtherPageSizeInBytes, typename OtherAllocator>
	void Assign(const TPagedArray<ElementType, OtherPageSizeInBytes, OtherAllocator>& Other)
	{
		Reset();
		Append(Other);
	}

	/*
	 * Constructs an element in place using the parameter arguments and adds it at the back of the container.
	 * This method returns a reference to the constructed element.
	 */
	template <typename... ArgsType>
	SizeType Emplace(ArgsType&&... Args)
	{
		GrowIfRequired();
		const SizeType ElementIndex = Num();
		const SizeType PageIndex = GetPageIndex(ElementIndex);
		Pages[PageIndex].Emplace(Forward<ArgsType>(Args)...);
		++Count;
		return ElementIndex;
	}

	/*
	 * Constructs an element in place using the parameter arguments and adds it at the back of the container.
	 * This method returns a reference to the constructed element.
	 */
	template <typename... ArgsType>
	[[nodiscard]] ElementType& Emplace_GetRef(ArgsType&&... Args)
	{
		GrowIfRequired();
		const SizeType PageIndex = GetPageIndex(Num());
		ElementType& Result = Pages[PageIndex].Emplace_GetRef(Forward<ArgsType>(Args)...);
		++Count;
		return Result;
	}

	/*
	 * Adds the parameter element at the back of the container.
	 */
	UE_FORCEINLINE_HINT SizeType Add(const ElementType& Element)
	{
		return Emplace(Element);
	}

	UE_FORCEINLINE_HINT SizeType Add(ElementType&& Element)
	{
		return Emplace(MoveTempIfPossible(Element));
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& Add_GetRef(const ElementType& Element)
	{
		return Emplace_GetRef(Element);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& Add_GetRef(ElementType&& Element)
	{
		return Emplace_GetRef(MoveTempIfPossible(Element));
	}

	UE_FORCEINLINE_HINT void Push(const ElementType& Element)
	{
		Emplace(Element);
	}

	UE_FORCEINLINE_HINT void Push(ElementType&& Element)
	{
		Emplace(MoveTempIfPossible(Element));
	}

	/*
	 * Removes the last element in the container.
	 */
	void Pop(EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		CheckValidIndex(0);
		const SizeType LastIndex = Num() - 1;
		const SizeType LastPageIndex = GetPageIndex(LastIndex);
		const SizeType LastIndexInPage = GetPageOffset(LastIndex);
		Pages[LastPageIndex].RemoveAt(LastIndexInPage, EAllowShrinking::No);
		if (AllowShrinking == EAllowShrinking::Yes && LastIndexInPage == 0)
		{
			Pages.SetNum(LastPageIndex);
		}
		--Count;
	}

	/**
	 * Removes the element at the parameter index position and swaps the last element if existent to the same position
	 * to ensure the range is contiguous. This method provides efficient removal O(1) but it doesn't preserve the
	 * insertion order.
	 */
	void RemoveAtSwap(SizeType Index, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		CheckValidIndex(Index);
		const SizeType TargetPageIndex = GetPageIndex(Index);
		const SizeType TargetIndexInPage = GetPageOffset(Index);
		const SizeType LastIndex = Num() - 1;
		const SizeType LastPageIndex = GetPageIndex(LastIndex);
		if (TargetPageIndex == LastPageIndex)
		{
			Pages[TargetPageIndex].RemoveAtSwap(TargetIndexInPage, EAllowShrinking::No);
			if (AllowShrinking == EAllowShrinking::Yes && Pages[TargetPageIndex].IsEmpty())
			{
				Pages.SetNum(TargetPageIndex);
			}
		}
		else
		{
			const SizeType LastIndexInPage = GetPageOffset(LastIndex);
			Pages[TargetPageIndex][TargetIndexInPage] = MoveTempIfPossible(Pages[LastPageIndex][LastIndexInPage]);
			Pages[LastPageIndex].RemoveAt(LastIndexInPage, EAllowShrinking::No);
		}
		--Count;
	}

	/*
	 * Empties the container effectively destroying all contained elements and releases any acquired storage.
	 */
	void Empty()
	{
		Pages.Empty();
		Count = 0;
	}

	/*
	 * Empties the container effectively destroying all contained elements and ensures storage for the parameter
	 * capacity. Storage is adjusted to meet the parameter capacity value.
	 */
	void Empty(SizeType InCapacity)
	{
		Reset();

		const SizeType PageCount = Pages.Num();
		const SizeType RequiredPageCount = NumRequiredPages(InCapacity);
		Pages.SetNum(RequiredPageCount);
		for (SizeType Index = PageCount; Index < RequiredPageCount; ++Index)
		{
			Pages[Index].Reserve(PageTraits::Capacity);
		}
	}

	/*
	 * Destroys all contained elements but doesn't release the container's storage.
	 */
	void Reset()
	{
		for (PageType& Page : Pages)
		{
			Page.Reset();
		}
		Count = 0;
	}

	/*
	 * Destroys all contained elements and ensures storage for the parameter capacity. Storage is only acquired
	 * if the current container's storage cannot meet the parameter capacity value.
	 */
	void Reset(SizeType InCapacity)
	{
		Reset();
		Reserve(InCapacity);
	}

	/*
	 * Reserves storage for the parameter element count.
	 */
	inline void Reserve(SizeType InCount)
	{
		check(InCount >= 0);
		GrowIfRequired(InCount);
	}

	/*
	 * Copies this container elements into the parameter destination array.
	 */
	template <typename AnyAllocator>
	void ToArray(TArray<ElementType, AnyAllocator>& OutDestination) const&
	{
		OutDestination.Reset();
		OutDestination.Reserve(Count);
		for (const PageType& Page : Pages)
		{
			if (Page.IsEmpty())
			{
				break;
			}
			OutDestination.Append(Page);
		}
	}

	/*
	 * Moves this container elements into the parameter destination array.
	 */
	template <typename AnyAllocator>
	void ToArray(TArray<ElementType, AnyAllocator>& OutDestination) &&
	{
		OutDestination.Reset();
		OutDestination.Reserve(Count);
		for (PageType& Page : Pages)
		{
			if (Page.IsEmpty())
			{
				break;
			}
			OutDestination.Append(MoveTemp(Page));
		}
		Count = 0;
	}

	/** STL IteratorType model compliance methods.*/
	[[nodiscard]] UE_FORCEINLINE_HINT ConstIteratorType begin() const
	{
		return GetIterator(0);
	}
	[[nodiscard]] UE_FORCEINLINE_HINT IteratorType begin()
	{
		return GetIterator(0);
	}
	[[nodiscard]] UE_FORCEINLINE_HINT ConstIteratorType end() const
	{
		return GetIterator(Count);
	}
	[[nodiscard]] UE_FORCEINLINE_HINT IteratorType end()
	{
		return GetIterator(Count);
	}

	// Friend operators
	[[nodiscard]] bool operator==(const TPagedArray& Right) const
	{
		if (Num() != Right.Num())
		{
			return false;
		}
		const SizeType MinPageCount = NumRequiredPages(Count);
		for (SizeType PageIndex = 0; PageIndex < MinPageCount; ++PageIndex)
		{
			if (Pages[PageIndex] != Right.Pages[PageIndex])
			{
				return false;
			}
		}
		return true;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TPagedArray& Right) const
	{
		return !(*this == Right);
	}

private:
	TArray<PageType, AllocatorType> Pages;
	SizeType Count = 0;

	[[nodiscard]] UE_FORCEINLINE_HINT ConstIteratorType GetIterator(SizeType Offset) const
	{
		return ConstIteratorType(InPlace, Pages.GetData(), Offset);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT IteratorType GetIterator(SizeType Offset)
	{
		return IteratorType(InPlace, Pages.GetData(), Offset);
	}

	/*
	 * Adds a new page and reserves its capacity predefined by its traits.
	 */
	PageType& AddPage()
	{
		PageType& Page = Pages.Emplace_GetRef();
		Page.Reserve(PageTraits::Capacity);
		return Page;
	}

	/*
	 * Copies the parameter element array of the parameter size into this container.
	 * This method assumes no previously existing content.
	 */
	void CopyToEmpty(const ElementType* InSource, SizeType InSize)
	{
		checkSlow(!Count);
		if (InSize)
		{
			Grow(InSize);
			CopyUnchecked(InSource, InSize);
		}
	}

	/*
	 * Copies the parameter element array of the parameter size into this container.
	 */
	void CopyUnchecked(const ElementType* InSource, SizeType InSize)
	{
		if (InSize)
		{
			SizeType PageIndex = GetPageIndex(Count);
			SizeType PendingCount = InSize;
			if (const SizeType PageOffset = GetPageOffset(Count))
			{
				const SizeType AppendCount = FMath::Min(PageTraits::Capacity - PageOffset, PendingCount);
				Pages[PageIndex++].Append(InSource, AppendCount);
				InSource += AppendCount;
				PendingCount -= AppendCount;
			}
			while (PendingCount >= PageTraits::Capacity)
			{
				Pages[PageIndex++].Append(InSource, PageTraits::Capacity);
				InSource += PageTraits::Capacity;
				PendingCount -= PageTraits::Capacity;
			}
			if (PendingCount)
			{
				Pages[PageIndex].Append(InSource, PendingCount);
			}
			Count += InSize;
		}
	}

	/*
	 * Grows the container's storage to the parameter capacity value allocating the required page count.
	 * This method assumes the container's current capacity is smaller than the parameter value.
	 * This method preserves existing data.
	 */
	void Grow(SizeType InCapacity)
	{
		checkSlow(Max() < InCapacity);
		const SizeType RequiredPageCount = NumRequiredPages(InCapacity);
		Pages.Reserve(RequiredPageCount);
		for (SizeType Index = Pages.Num(); Index < RequiredPageCount; ++Index)
		{
			AddPage();
		}
	}

	/*
	 * Grows the container by adding a new page if full.
	 * This method preserves existing data.
	 */
	void GrowIfRequired()
	{
		if (Count == Max())
		{
			AddPage();
		}
	}

	/*
	 * Grows the container's storage to meet the parameter capacity value.
	 * This method preserves existing data.
	 */
	void GrowIfRequired(SizeType InCapacity)
	{
		if (Max() < InCapacity)
		{
			Grow(InCapacity);
		}
	}

	UE_FORCEINLINE_HINT void CheckValidIndex(SizeType Index) const
	{
		checkf((Index >= 0) & (Index < Count), TEXT("Parameter index %d exceeds container size %d"), Index, Count);
	}
};

/** Serializer. */
template <typename ElementType, int32 PageSizeInBytes, typename AllocatorType>
UE_FORCEINLINE_HINT FArchive& operator<<(FArchive& Ar, TPagedArray<ElementType, PageSizeInBytes, AllocatorType>& InOutPagedArray)
{
	return UE::Core::PagedArray::Private::Serialize(Ar, InOutPagedArray);
}

/** Structured archive serializer. */
template <typename ElementType, int32 PageSizeInBytes, typename AllocatorType>
UE_FORCEINLINE_HINT void operator<<(FStructuredArchive::FSlot Slot, TPagedArray<ElementType, PageSizeInBytes, AllocatorType>& InOutPagedArray)
{
	UE::Core::PagedArray::Private::SerializeStructured(Slot, InOutPagedArray);
}
