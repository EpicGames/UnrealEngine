// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UnrealTemplate.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PlatformMisc.h"
#include <type_traits>

// WARNING: This queue is planned for deprecation in favor of TSpscQueue or TMpscQueue

/**
 * Enumerates concurrent queue modes.
 */
enum class EQueueMode
{
	/** Multiple-producers, single-consumer queue. */
	Mpsc,

	/** Single-producer, single-consumer queue. */
	Spsc,

	/** Single-threaded - no guarantees of concurrent safety. */
	SingleThreaded,
};


/**
 * Template for queues.
 *
 * This template implements an unbounded non-intrusive queue using a lock-free linked
 * list that stores copies of the queued items. The template can operate in three modes:
 * Multiple-producers single-consumer (MPSC), Single-producer single-consumer (SPSC), and Single-threaded.
 *
 * The queue is thread-safe in both MPSC and SPSC modes. The Dequeue() method ensures thread-safety by
 * writing it in a way that does not depend on possible instruction reordering on the CPU.
 * The Enqueue() method uses an atomic compare-and-swap in multiple-producers scenarios.
 *
 * The queue is not thread-safe in single-threaded mode, as the name suggests.
 *
 * @param T The type of items stored in the queue.
 * @param Mode The queue mode (single-producer, single-consumer by default).
 * @todo gmp: Implement node pooling.
 */
template<typename T, EQueueMode Mode = EQueueMode::Spsc>
class TQueue
{
public:
	using FElementType = T;

	/** Default constructor. */
	[[nodiscard]] TQueue()
	{
		Head = Tail = new TNode();
	}

	/** Destructor. */
	~TQueue()
	{
		while (Tail != nullptr)
		{
			TNode* Node = Tail;
			Tail = Tail->NextNode;

			delete Node;
		}
	}

	UE_NONCOPYABLE(TQueue);

	/**
	 * Removes and returns the item from the tail of the queue.
	 *
	 * @param OutValue Will hold the returned value.
	 * @return true if a value was returned, false if the queue was empty.
	 * @note To be called only from consumer thread.
	 * @see Empty, Enqueue, IsEmpty, Peek, Pop
	 */
	bool Dequeue(FElementType& OutItem)
	{
		TNode* Popped = Tail->NextNode;

		if (Popped == nullptr)
		{
			return false;
		}

		if constexpr (Mode != EQueueMode::SingleThreaded)
		{
			TSAN_AFTER(&Tail->NextNode);
		}

		OutItem = MoveTemp(Popped->Item);

		TNode* OldTail = Tail;
		Tail = Popped;
		Tail->Item = FElementType();
		delete OldTail;

		return true;
	}

	/**
	 * Empty the queue, discarding all items.
	 *
	 * @note To be called only from consumer thread.
	 * @see Dequeue, IsEmpty, Peek, Pop
	 */
	void Empty()
	{
		while (Pop());
	}

	/**
	 * Adds an item to the head of the queue.
	 *
	 * @param Item The item to add.
	 * @return true if the item was added, false otherwise.
	 * @note To be called only from producer thread(s).
	 * @see Dequeue, Pop
	 */
	bool Enqueue(const FElementType& Item)
	{
		TNode* NewNode = new TNode(Item);

		if (NewNode == nullptr)
		{
			return false;
		}

		TNode* OldHead;

		if constexpr (Mode == EQueueMode::Mpsc)
		{
			OldHead = (TNode*)FPlatformAtomics::InterlockedExchangePtr((void**)&Head, NewNode);
			TSAN_BEFORE(&OldHead->NextNode);
			FPlatformAtomics::InterlockedExchangePtr((void**)&OldHead->NextNode, NewNode);
		}
		else
		{
			OldHead = Head;
			Head = NewNode;

			if constexpr (Mode == EQueueMode::Spsc)
			{
				TSAN_BEFORE(&OldHead->NextNode);
				FPlatformMisc::MemoryBarrier();
			}

			OldHead->NextNode = NewNode;
		}

		return true;
	}

	/**
	 * Adds an item to the head of the queue.
	 *
	 * @param Item The item to add.
	 * @return true if the item was added, false otherwise.
	 * @note To be called only from producer thread(s).
	 * @see Dequeue, Pop
	 */
	bool Enqueue(FElementType&& Item)
	{
		TNode* NewNode = new TNode(MoveTemp(Item));

		if (NewNode == nullptr)
		{
			return false;
		}

		TNode* OldHead;

		if constexpr (Mode == EQueueMode::Mpsc)
		{
			OldHead = (TNode*)FPlatformAtomics::InterlockedExchangePtr((void**)&Head, NewNode);
			TSAN_BEFORE(&OldHead->NextNode);
			FPlatformAtomics::InterlockedExchangePtr((void**)&OldHead->NextNode, NewNode);
		}
		else
		{
			OldHead = Head;
			Head = NewNode;

			if constexpr (Mode == EQueueMode::Spsc)
			{
				TSAN_BEFORE(&OldHead->NextNode);
				FPlatformMisc::MemoryBarrier();
			}

			OldHead->NextNode = NewNode;
		}

		return true;
	}

	/**
	 * Checks whether the queue is empty.
	 *
	 * @return true if the queue is empty, false otherwise.
	 * @note To be called only from consumer thread.
	 * @see Dequeue, Empty, Peek, Pop
	 */
	[[nodiscard]] bool IsEmpty() const
	{
		return (Tail->NextNode == nullptr);
	}

	/**
	 * Peeks at the queue's tail item without removing it.
	 *
	 * @param OutItem Will hold the peeked at item.
	 * @return true if an item was returned, false if the queue was empty.
	 * @note To be called only from consumer thread.
	 * @see Dequeue, Empty, IsEmpty, Pop
	 */
	bool Peek(FElementType& OutItem) const
	{
		if (Tail->NextNode == nullptr)
		{
			return false;
		}

		OutItem = Tail->NextNode->Item;

		return true;
	}

	/**
	 * Peek at the queue's tail item without removing it.
	 *
	 * This version of Peek allows peeking at a queue of items that do not allow
	 * copying, such as TUniquePtr.
	 *
	 * @return Pointer to the item, or nullptr if queue is empty
	 */
	[[nodiscard]] FElementType* Peek()
	{
		if (Tail->NextNode == nullptr)
		{
			return nullptr;
		}

		return &Tail->NextNode->Item;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT const FElementType* Peek() const
	{
		return const_cast<TQueue*>(this)->Peek();
	}

	/**
	 * Removes the item from the tail of the queue.
	 *
	 * @return true if a value was removed, false if the queue was empty.
	 * @note To be called only from consumer thread.
	 * @see Dequeue, Empty, Enqueue, IsEmpty, Peek
	 */
	bool Pop()
	{
		TNode* Popped = Tail->NextNode;

		if (Popped == nullptr)
		{
			return false;
		}

		if constexpr (Mode != EQueueMode::SingleThreaded)
		{
			TSAN_AFTER(&Tail->NextNode);
		}

		TNode* OldTail = Tail;
		Tail = Popped;
		Tail->Item = FElementType();
		delete OldTail;

		return true;
	}

private:

	/** Structure for the internal linked list. */
	struct TNode;
	using TNodeVolatilePtr = std::conditional_t<Mode == EQueueMode::SingleThreaded, TNode*, TNode* volatile>;

	struct TNode
	{
		/** Holds a pointer to the next node in the list. */
		TNodeVolatilePtr NextNode = nullptr;

		/** Holds the node's item. */
		FElementType Item;

		/** Default constructor. */
		[[nodiscard]] TNode() = default;

		/** Creates and initializes a new node. */
		[[nodiscard]] explicit TNode(const FElementType& InItem)
			: Item(InItem)
		{
		}

		/** Creates and initializes a new node. */
		[[nodiscard]] explicit TNode(FElementType&& InItem)
			: Item(MoveTemp(InItem))
		{
		}
	};

	/** Holds a pointer to the head of the list. */
	MS_ALIGN(16) TNodeVolatilePtr Head GCC_ALIGN(16);

	/** Holds a pointer to the tail of the list. */
	TNode* Tail;
};
