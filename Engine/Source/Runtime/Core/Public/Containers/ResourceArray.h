// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Serialization/MemoryLayout.h"

/** An element type independent interface for uploading an array of resource data. */
struct FResourceArrayUploadInterface
{
	virtual ~FResourceArrayUploadInterface() = default;

	/** Returns a pointer to the resource data. */
	[[nodiscard]] virtual const void* GetResourceData() const = 0;

	/** Returns size of resource data allocation */
	[[nodiscard]] virtual uint32 GetResourceDataSize() const = 0;

	template<typename TElement>
	TConstArrayView<TElement> GetResourceDataView() const
	{
		return TConstArrayView<TElement>(reinterpret_cast<const TElement*>(GetResourceData()), GetResourceDataSize());
	}

	/** Called on non-UMA systems after the RHI has copied the resource data, and no longer needs the CPU's copy. */
	virtual void Discard() = 0;
};

/** Utility to do a simple upload of data from an array managed by the caller. */
struct FResourceArrayUploadArrayView : public FResourceArrayUploadInterface
{
	const void* const Data;
	const uint32 SizeInBytes;

	FResourceArrayUploadArrayView() = delete;

	[[nodiscard]] explicit FResourceArrayUploadArrayView(const void* InData, uint32 InSizeInBytes)
		: Data(InData)
		, SizeInBytes(InSizeInBytes)
	{
	}

	template<typename ElementType>
	[[nodiscard]] explicit FResourceArrayUploadArrayView(TConstArrayView<ElementType> View)
		: Data(View.GetData())
		, SizeInBytes(View.Num() * View.GetTypeSize())
	{
	}

	template<typename ElementType, typename AllocatorType>
	[[nodiscard]] explicit FResourceArrayUploadArrayView(const TArray<ElementType, AllocatorType>& InArray)
		: FResourceArrayUploadArrayView(MakeArrayView(InArray))
	{
	}

	// FResourceArrayUploadInterface
	[[nodiscard]] virtual const void* GetResourceData() const final
	{
		return Data;
	}

	[[nodiscard]] virtual uint32 GetResourceDataSize() const final
	{
		return SizeInBytes;
	}

	virtual void Discard() final
	{
		// do nothing
	}
};

/**
 * An element type independent interface to the resource array.
 */
class FResourceArrayInterface : public FResourceArrayUploadInterface
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FResourceArrayInterface, CORE_API, Abstract);
public:

	/**
	 * @return true if the resource array is static and shouldn't be modified
	 */
	[[nodiscard]] virtual bool IsStatic() const = 0;

	/**
	 * @return true if the resource keeps a copy of its resource data after the RHI resource has been created
	 */
	[[nodiscard]] virtual bool GetAllowCPUAccess() const = 0;

	/** 
	 * Sets whether the resource array will be accessed by CPU. 
	 */
	virtual void SetAllowCPUAccess( bool bInNeedsCPUAccess ) = 0;
};


/**
 * Allows for direct GPU mem allocation for bulk resource types.
 */
class FResourceBulkDataInterface
{
public:

	virtual ~FResourceBulkDataInterface() = default;

	/** 
	 * @return ptr to the resource memory which has been preallocated
	 */
	[[nodiscard]] virtual const void* GetResourceBulkData() const = 0;

	/** 
	 * @return size of resource memory
	 */
	[[nodiscard]] virtual uint32 GetResourceBulkDataSize() const = 0;

	template<typename TElement>
	[[nodiscard]] TConstArrayView<TElement> GetBulkDataView() const
	{
		return TConstArrayView<TElement>(reinterpret_cast<const TElement*>(GetResourceBulkData()), GetResourceBulkDataSize());
	}

	/**
	 * Free memory after it has been used to initialize RHI resource 
	 */
	virtual void Discard() = 0;
};

/** Utility to do a simple upload of data from an array managed by the caller. */
struct FResourceBulkDataArrayView : public FResourceBulkDataInterface
{
	const void* const Data;
	const uint32 SizeInBytes;

	FResourceBulkDataArrayView() = delete;

	[[nodiscard]] explicit FResourceBulkDataArrayView(const void* InData, uint32 InSizeInBytes)
		: Data(InData)
		, SizeInBytes(InSizeInBytes)
	{
	}

	template<typename ElementType>
	[[nodiscard]] explicit FResourceBulkDataArrayView(TConstArrayView<ElementType> View)
		: Data(View.GetData())
		, SizeInBytes(View.Num()* View.GetTypeSize())
	{
	}

	template<typename ElementType, typename AllocatorType>
	[[nodiscard]] explicit FResourceBulkDataArrayView(const TArray<ElementType, AllocatorType>& InArray)
		: FResourceBulkDataArrayView(MakeArrayView(InArray))
	{
	}

	// FResourceArrayUploadInterface
	[[nodiscard]] virtual const void* GetResourceBulkData() const final
	{
		return Data;
	}

	[[nodiscard]] virtual uint32 GetResourceBulkDataSize() const final
	{
		return SizeInBytes;
	}

	virtual void Discard() final
	{
		// do nothing
	}
};


/**
 * Allows for direct GPU mem allocation for texture resource.
 */
class FTexture2DResourceMem : public FResourceBulkDataInterface
{
public:

	/**
	 * @param MipIdx index for mip to retrieve
	 * @return ptr to the offset in bulk memory for the given mip
	 */
	[[nodiscard]] virtual void* GetMipData(int32 MipIdx) = 0;

	/**
	 * @return total number of mips stored in this resource
	 */
	[[nodiscard]] virtual int32	GetNumMips() = 0;

	/** 
	 * @return width of texture stored in this resource
	 */
	[[nodiscard]] virtual int32 GetSizeX() = 0;

	/** 
	 * @return height of texture stored in this resource
	 */
	[[nodiscard]] virtual int32 GetSizeY() = 0;

	/**
	 * @return Whether the resource memory is properly allocated or not.
	 */
	[[nodiscard]] virtual bool IsValid() = 0;

	/**
	 * @return Whether the resource memory has an async allocation request and it's been completed.
	 */
	[[nodiscard]] virtual bool HasAsyncAllocationCompleted() const = 0;

	/** Blocks the calling thread until the allocation has been completed. */
	virtual void FinishAsyncAllocation() = 0;

	/** Cancels any async allocation. */
	virtual void CancelAsyncAllocation() = 0;

	virtual ~FTexture2DResourceMem() = default;
};

