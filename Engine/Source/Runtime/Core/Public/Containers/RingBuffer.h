// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/ArrayView.h"
#include "Templates/IsPODType.h"
#include "Templates/MemoryOps.h"
#include "Templates/UnrealTemplate.h"

#include <type_traits>

template< typename ContainerType, typename ElementType, typename SizeType>
class TRingBufferIterator
{
public:
	[[nodiscard]] TRingBufferIterator(ContainerType& InContainer, SizeType StartIndex = 0)
		: Container(InContainer)
		, Index(StartIndex)
	{
	}

	/** Advances iterator to the next element in the container. */
	TRingBufferIterator& operator++()
	{
		++Index;
		return *this;
	}
	TRingBufferIterator operator++(int)
	{
		TRingBufferIterator Tmp(*this);
		++Index;
		return Tmp;
	}

	/** Moves iterator to the previous element in the container. */
	TRingBufferIterator& operator--()
	{
		--Index;
		return *this;
	}
	TRingBufferIterator operator--(int)
	{
		TRingBufferIterator Tmp(*this);
		--Index;
		return Tmp;
	}

	/** iterator arithmetic support */
	TRingBufferIterator& operator+=(SizeType Offset)
	{
		Index += Offset;
		return *this;
	}

	TRingBufferIterator operator+(SizeType Offset) const
	{
		TRingBufferIterator Tmp(*this);
		return Tmp += Offset;
	}

	TRingBufferIterator& operator-=(SizeType Offset)
	{
		return *this += -Offset;
	}

	TRingBufferIterator operator-(SizeType Offset) const
	{
		TRingBufferIterator Tmp(*this);
		return Tmp -= Offset;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& operator*() const
	{
		return Container[Index];
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType* operator->() const
	{
		return &Container[Index];
	}

	/** conversion to "bool" returning true if the iterator has not reached the last element. */
	[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
	{
		return Container.IsValidIndex(Index);
	}

	/** Returns an index to the current element. */
	[[nodiscard]] SizeType GetIndex() const
	{
		return Index;
	}

	/** Resets the iterator to the first element. */
	void Reset()
	{
		Index = 0;
	}

	/** Sets the iterator to one past the last element. */
	void SetToEnd()
	{
		Index = Container.Num();
	}

	/** Removes current element in array. This invalidates the current iterator value and it must be incremented */
	void RemoveCurrent()
	{
		Container.RemoveAt(Index);
		--Index;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TRingBufferIterator& Rhs) const { return &Container == &Rhs.Container && Index == Rhs.Index; }
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TRingBufferIterator& Rhs) const { return &Container != &Rhs.Container || Index != Rhs.Index; }

private:

	ContainerType& Container;
	SizeType      Index;
};

/**
 * RingBuffer - an array with a Front and Back pointer and with implicit wraparound to the beginning of the array when reaching the end of the array when iterating from Front to Back
 * Useful for providing O(1) push/pop at the end of the array (for Queue or Stack) while still having high cache coherency during iteration.
 * Not thread-safe; caller must ensure there is no simultaneous access from multiple threads.
 *
 * Implementation Details:
 * Relies on unsigned arithmetics and ever-increasing Front and Back indices to avoid having to store an extra element or maintain explicit empty state.
 * Capacity will always be rounded up to the next power of two, to provide rapid masking of the index.
 */
template<typename T, typename AllocatorT = FDefaultAllocator>
class TRingBuffer
{
public:
	/** Type of the Elements in the container. */
	typedef T ElementType;
	/* The Allocator type being used */
	typedef AllocatorT Allocator;
	using ElementAllocatorType = std::conditional_t<
		Allocator::NeedsElementType,
		typename Allocator::template ForElementType<ElementType>,
		typename Allocator::ForAnyElementType
	>;
	/** Type used to request values at a given index in the container. */
	typedef std::make_signed_t<typename Allocator::SizeType> IndexType;
	/** Type used to communicate size and capacity and counts */
	typedef std::make_unsigned_t<typename Allocator::SizeType> SizeType;
	/** Iterator type used for ranged-for traversal. */
	typedef TRingBufferIterator<TRingBuffer, ElementType, typename Allocator::SizeType> TIterator;
	typedef TRingBufferIterator<const TRingBuffer, const ElementType, typename Allocator::SizeType> TConstIterator;
private:
	/**
	 * Type used for variables that are indexes into the underlying storage.
	 * StorageModuloType types are offsets from the beginning of storage, and any bits outside IndexMask are discarded.
	 * They may have added on multiples of the capacity due to wrapping around, but will be interpreted as pointing to the value at (value & IndexMask).
	 * StorageModuloTypes are also allowed to underflow/overflow their integer storage type; the only constraint is that X - Front <= Capacity for all valid indexes and for AfterBack.
	 */
	typedef std::make_unsigned_t<typename Allocator::SizeType> StorageModuloType;
public:

	/** Construct Empty Queue with capacity 0. */
	[[nodiscard]] TRingBuffer()
		: IndexMask(static_cast<StorageModuloType>(-1))
		, Front(0u)
		, AfterBack(0u)
	{
	}

	/** Construct Empty Queue with the given initial requested capacity. */
	[[nodiscard]] explicit TRingBuffer(SizeType InitialCapacity)
		: TRingBuffer()
	{
		Reserve(InitialCapacity);
	}

	/** Construct a Queue with initial state (from front to back) equal to the given list. */
	[[nodiscard]] TRingBuffer(std::initializer_list<ElementType> InitList)
		: TRingBuffer(static_cast<SizeType>(InitList.size()))
	{
		for (const ElementType& Element : InitList)
		{
			Add(Element);
		}
	}

	[[nodiscard]] TRingBuffer(TRingBuffer&& Other)
		: TRingBuffer()
	{
		*this = MoveTemp(Other);
	}

	TRingBuffer& operator=(TRingBuffer&& Other)
	{
		if (this == &Other)
		{
			return *this;
		}
		Empty();
		AllocatorInstance.MoveToEmpty(Other.AllocatorInstance);
		Swap(IndexMask, Other.IndexMask);
		Swap(Front, Other.Front);
		Swap(AfterBack, Other.AfterBack);
		return *this;
	}

	[[nodiscard]] TRingBuffer(const TRingBuffer& Other)
		:TRingBuffer()
	{
		*this = Other;
	}

	TRingBuffer& operator=(const TRingBuffer& Other)
	{
		Empty(Other.Max());
		for (const ElementType& Element : Other)
		{
			Add(Element);
		}
		return *this;
	}

	~TRingBuffer()
	{
		Empty();
	}

	/** Returns true if the RingBuffer is empty. */
	[[nodiscard]] bool IsEmpty() const
	{
		return AfterBack == Front;
	}

	/** Gets the number of elements in the RingBuffer. */
	[[nodiscard]] IndexType Num() const
	{
		return AfterBack - Front;
	}

	/** Current allocated Capacity, note this will always be a power of two, or the special case 0. */
	[[nodiscard]] IndexType Max() const
	{
		return static_cast<SizeType>(IndexMask + 1);
	}

	/** Set the capacity to the maximum of the current capacity and the (next power of two greater than or equal to) the given capacity. */
	void Reserve(SizeType RequiredCapacity)
	{
		SizeType NewCapacity = NormalizeCapacity(RequiredCapacity);
		if (NewCapacity <= static_cast<SizeType>(Max()))
		{
			return;
		}
		Reallocate(NewCapacity);
	}

	/** Set the capacity to the minimum power of two (or 0) greater than or equal to the current number of elements in the RingBuffer. */
	void Trim()
	{
		SizeType NewCapacity = NormalizeCapacity(Num());
		Reallocate(NewCapacity);
	}

	/** Empty the RingBuffer, destructing any elements in the RingBuffer but not releasing the RingBuffer's storage. */
	void Reset()
	{
		PopFrontNoCheck(Num());
		AfterBack = 0;
		Front = 0;
	}

	/** Empty the RingBuffer, destructing any elements and releasing the RingBuffer's storage.  Sets the new capacity after release to the given capacity. */
	void Empty(SizeType Capacity = 0)
	{
		Reset();
		SizeType NewCapacity = NormalizeCapacity(Capacity);
		Reallocate(NewCapacity);
	}

	/** Add a new element after the back pointer of the RingBuffer, resizing if necessary.  The new element is move constructed from the argument. Returns the index of the added element. */
	IndexType Add(ElementType&& Element)
	{
		IndexType ResultIndex = AddUninitialized();
		ElementType& Result = GetAtIndexNoCheck(ResultIndex);
		::new ((void*)&Result) ElementType(MoveTempIfPossible(Element));
		return ResultIndex;
	}

	/** Add a new element after the back pointer of the RingBuffer, resizing if necessary.  The new element is move constructed from the argument. Returns a reference to the added element. */
	[[nodiscard]] ElementType& Add_GetRef(ElementType&& Element)
	{
		ElementType& Result = AddUninitialized_GetRef();
		::new ((void*)&Result) ElementType(MoveTempIfPossible(Element));
		return Result;
	}

	/** Add a new element after the back pointer of the RingBuffer, resizing if necessary.  The new element is copy constructed from the argument. Returns the index to the added element. */
	IndexType Add(const ElementType& Element)
	{
		IndexType ResultIndex = AddUninitialized();
		ElementType& Result = GetAtIndexNoCheck(ResultIndex);
		::new ((void*)&Result) ElementType(Element);
		return ResultIndex;
	}

	/** Add a new element after the back pointer of the RingBuffer, resizing if necessary.  The new element is copy constructed from the argument. Returns a reference to the added element. */
	[[nodiscard]] ElementType& Add_GetRef(const ElementType& Element)
	{
		IndexType ResultIndex = AddUninitialized();
		ElementType& Result = GetAtIndexNoCheck(ResultIndex);
		::new ((void*)&Result) ElementType(Element);
		return Result;
	}

	/** Add a new element after the back pointer of the RingBuffer, resizing if necessary.  The new element is constructed from the given arguments. Returns the index to the added element. */
	template <typename... ArgsType>
	IndexType Emplace(ArgsType&&... Args)
	{
		IndexType ResultIndex = AddUninitialized();
		ElementType& Result = GetAtIndexNoCheck(ResultIndex);
		::new ((void*)&Result) ElementType(Forward<ArgsType>(Args)...);
		return ResultIndex;
	}

	/** Add a new element after the back pointer of the RingBuffer, resizing if necessary.  The new element is constructed from the given arguments. Returns a reference to the added element. */
	template <typename... ArgsType>
	[[nodiscard]] ElementType& Emplace_GetRef(ArgsType&&... Args)
	{
		ElementType& Result = AddUninitialized_GetRef();
		::new ((void*)&Result) ElementType(Forward<ArgsType>(Args)...);
		return Result;
	}

	/** Add a new element after the back pointer of the RingBuffer, resizing if necessary.  The constructor is not called on the new element and its values in memory are arbitrary. Returns the index to the added element. */
	IndexType AddUninitialized()
	{
		ConditionalIncrementCapacity();
		// Note this increment may overflow and set AfterBack = 0.  This overflow is legal;
		// the constraint ((AfterBack - Front) == Num()) will still be true despite Front and AfterBack being
		// on opposite sides of 0.
		IndexType Result = static_cast<IndexType>(AfterBack++ - Front); 
		SlackTrackerNumChanged();
		return Result;
	}

	/** Add a new element after the back pointer of the RingBuffer, resizing if necessary.  The constructor is not called on the new element and its values in memory are arbitrary. Returns a reference to the added element. */
	[[nodiscard]] ElementType& AddUninitialized_GetRef()
	{
		return GetAtIndexNoCheck(AddUninitialized());
	}

	/**
	 * Append elements from a range onto the back pointer of the RingBuffer, resizing if necessary.
	 * Each element is move-constructed into the RingBuffer; source elements may therefore be modified.
	 */
	void MoveAppendRange(ElementType* OtherData, SizeType OtherNum)
	{
		if (OtherNum == 0)
		{
			return;
		}
		check(OtherData);
		const SizeType OldNum = static_cast<SizeType>(Num());
		const SizeType NewNum = OldNum + OtherNum;
		checkf(NewNum > OldNum, TEXT("Overflow: Num() == %d, OtherNum == %d, NewNum == %d"), OldNum, OtherNum, NewNum);
		Reserve(NewNum);
		const SizeType LocalIndexMask = IndexMask;

		const SizeType MoveRangeStart = AfterBack;
		const SizeType MoveRangeEnd = AfterBack + OtherNum;
		const StorageModuloType MaskedMoveRangeStart = MoveRangeStart & LocalIndexMask;
		const StorageModuloType MaskedMoveRangeEnd = MoveRangeEnd & LocalIndexMask;
		ElementType* const Data = GetStorage();
		if (MaskedMoveRangeStart >= MaskedMoveRangeEnd)
		{
			// We should not be reaching capacity and overwriting front
			check(MaskedMoveRangeEnd <= (Front & LocalIndexMask) && (Front & LocalIndexMask) <= MaskedMoveRangeStart);
			const SizeType FirstMoveCount = (LocalIndexMask + 1 - MaskedMoveRangeStart);
			MoveConstructItems(Data + MaskedMoveRangeStart, OtherData, FirstMoveCount);
			MoveConstructItems(Data, OtherData + FirstMoveCount, MaskedMoveRangeEnd);
		}
		else
		{
			// We should not be reaching capacity and overwriting front
			check((Front & LocalIndexMask) <= MaskedMoveRangeStart || MaskedMoveRangeEnd <= (Front & LocalIndexMask));
			MoveConstructItems(Data + MaskedMoveRangeStart, OtherData, OtherNum);
		}
		AfterBack += OtherNum;
		SlackTrackerNumChanged();
	}

	/** Add a new element before the front pointer of the RingBuffer, resizing if necessary.  The new element is move constructed from the argument. Returns the index of the added element. */
	IndexType AddFront(ElementType&& Element)
	{
		IndexType ResultIndex = AddFrontUninitialized();
		ElementType& Result = GetAtIndexNoCheck(ResultIndex);
		::new ((void*)&Result) ElementType(MoveTempIfPossible(Element));
		return ResultIndex;
	}

	/** Add a new element before the front pointer of the RingBuffer, resizing if necessary.  The new element is move constructed from the argument. Returns a reference to the added element. */
	[[nodiscard]] ElementType& AddFront_GetRef(ElementType&& Element)
	{
		ElementType& Result = AddFrontUninitialized_GetRef();
		::new ((void*)&Result) ElementType(MoveTempIfPossible(Element));
		return Result;
	}

	/** Add a new element before the front pointer of the RingBuffer, resizing if necessary.  The new element is copy constructed from the argument. Returns the index to the added element. */
	IndexType AddFront(const ElementType& Element)
	{
		IndexType ResultIndex = AddFrontUninitialized();
		ElementType& Result = GetAtIndexNoCheck(ResultIndex);
		::new ((void*)&Result) ElementType(Element);
		return ResultIndex;
	}

	/** Add a new element before the front pointer of the RingBuffer, resizing if necessary.  The new element is copy constructed from the argument. Returns a reference to the added element. */
	[[nodiscard]] ElementType& AddFront_GetRef(const ElementType& Element)
	{
		IndexType ResultIndex = AddFrontUninitialized();
		ElementType& Result = GetAtIndexNoCheck(ResultIndex);
		::new ((void*)&Result) ElementType(Element);
		return Result;
	}

	/** Add a new element before the front pointer of the RingBuffer, resizing if necessary.  The new element is constructed from the given arguments. Returns the index to the added element. */
	template <typename... ArgsType>
	IndexType EmplaceFront(ArgsType&&... Args)
	{
		IndexType ResultIndex = AddFrontUninitialized();
		ElementType& Result = GetAtIndexNoCheck(ResultIndex);
		::new ((void*)&Result) ElementType(Forward<ArgsType>(Args)...);
		return ResultIndex;
	}

	/** Add a new element before the front pointer of the RingBuffer, resizing if necessary.  The new element is constructed from the given arguments. Returns a reference to the added element. */
	template <typename... ArgsType>
	[[nodiscard]] ElementType& EmplaceFront_GetRef(ArgsType&&... Args)
	{
		ElementType& Result = AddFrontUninitialized_GetRef();
		::new ((void*)&Result) ElementType(Forward<ArgsType>(Args)...);
		return Result;
	}

	/** Add a new element before the front pointer of the RingBuffer, resizing if necessary.  The constructor is not called on the new element and its values in memory are arbitrary. Returns the index to the added element. */
	IndexType AddFrontUninitialized()
	{
		ConditionalIncrementCapacity();
		// Note this decrement may underflow and set Front = 0xffffffff.  This underflow is legal;
		// the constraint ((AfterBack - Front) == Num()) will still be true despite Front and AfterBack being on
		// opposite sides of 0.
		--Front;
		SlackTrackerNumChanged();
		return 0;
	}

	/** Add a new element before the front pointer of the RingBuffer, resizing if necessary.  The constructor is not called on the new element and its values in memory are arbitrary. Returns a reference to the added element. */
	[[nodiscard]] ElementType& AddFrontUninitialized_GetRef()
	{
		return GetAtIndexNoCheck(AddFrontUninitialized());
	}

	/** Return a reference to the element at the front pointer of the RingBuffer.  Invalid to call on an empty RingBuffer. */
	[[nodiscard]] ElementType& First()
	{
		return (*this)[0];
	}

	/** Return a const reference to the element at the front pointer of the RingBuffer.  Invalid to call on an empty RingBuffer. */
	[[nodiscard]] const ElementType& First() const
	{
		return (*this)[0];
	}

	/** Return a reference to the element at the back pointer of the RingBuffer.  Invalid to call on an empty RingBuffer. */
	[[nodiscard]] ElementType& Last()
	{
		return (*this)[Num() - 1];
	}

	/** Return a const reference to the element at the back pointer of the RingBuffer.  Invalid to call on an empty RingBuffer. */
	[[nodiscard]] const ElementType& Last() const
	{
		return (*this)[Num() - 1];
	}

	/** Pop the given number of elements (default: 1) from the front pointer of the RingBuffer.  Invalid to call with a number of elements greater than the current number of elements in the RingBuffer. */
	void PopFront(SizeType PopCount=1)
	{
		PopRangeCheck(PopCount);
		PopFrontNoCheck(PopCount);
	}

	/** Unsafely pop the given number of arguments (default: 1) from the front pointer of the RingBuffer.  Invalid to call (and does not warn) with a number of elements greater than the current number of elements in the RingBuffer. */
	void PopFrontNoCheck(SizeType PopCount=1)
	{
		DestructRange(Front, Front + PopCount);
		// Note this increment may overflow (wrapping around to 0xffffffff) if AfterBack has already underflowed; this is valid.
		Front += PopCount;
		SlackTrackerNumChanged();
	}

	/* Pop one element from the front pointer of the RingBuffer and return the popped value. Invalid to call when the RingBuffer is empty. */
	ElementType PopFrontValue()
	{
		PopRangeCheck(1);
		ElementType Result(MoveTemp(First()));
		PopFrontNoCheck(1);
		return Result;
	}

	/** Pop the given number of arguments (default: 1) from the back pointer of the RingBuffer.  Invalid to call with a number of elements greater than the current number of elements in the RingBuffer. */
	void Pop(SizeType PopCount=1)
	{
		PopRangeCheck(PopCount);
		PopNoCheck(PopCount);
	}

	/** Pop the given number of elements (default: 1) from the back pointer of the RingBuffer.  Invalid to call (and does not warn) with a number of elements greater than the current number of elements in the RingBuffer. */
	void PopNoCheck(SizeType PopCount=1)
	{
		DestructRange(AfterBack - PopCount, AfterBack);
		// Note this decrement may underflow (wrapping around to 0xffffffff) if Front has already underflowed; this is valid.
		AfterBack -= PopCount;
		SlackTrackerNumChanged();
	}

	/* Pop one element from the back pointer of the RingBuffer and return the popped value. Invalid to call when the RingBuffer is empty. */
	ElementType PopValue()
	{
		PopRangeCheck(1);
		ElementType Result(MoveTemp(Last()));
		PopNoCheck(1);
		return Result;
	}

	/** Move the value at the given index into the front pointer of the RingBuffer, and shift all elements ahead of it down by one to make room for it.  Invalid to call with a negative index or index greater than the number of elements in the RingBuffer. */
	void ShiftIndexToFront(IndexType Index)
	{
		RangeCheck(Index);
		if (Index == 0)
		{
			return;
		}

		ShiftLastToFirst(Front, Front + Index, -1);
	}

	/** Move the value at the given index into the back pointer of the RingBuffer, and shift all elements behind of it up by one to make room for it.  Invalid to call with an index less than 0 or greater than or equal to the number of elements in the RingBuffer. */
	void ShiftIndexToBack(IndexType Index)
	{
		IndexType LocalNum = Num();
		RangeCheck(Index);
		if (Index == LocalNum - 1)
		{
			return;
		}

		ShiftLastToFirst(AfterBack - 1, Front + Index, 1); // Note 0 <= AfterBack - 1 - (Front + Index) is <= (Num()-1) because AfterBack - Front - 1 == Num()-1 and 0 <= Index <= Num()-1
	}

	/** begin iterator for ranged-for */
	[[nodiscard]] TIterator begin()
	{
		return TIterator(*this);
	}

	/** begin iterator for const ranged-for */
	[[nodiscard]] TConstIterator begin() const
	{
		return TConstIterator(*this);
	}

	/** end iterator for ranged-for */
	[[nodiscard]] TIterator end()
	{
		return TIterator(*this, Num());
	}

	/** end iterator for const ranged-for */
	[[nodiscard]] TConstIterator end() const
	{
		return TConstIterator(*this, Num());
	}

	/** Tests if index is valid, i.e. greater than or equal to zero, and less than the number of elements in the array. */
	[[nodiscard]] bool IsValidIndex(IndexType Index) const
	{
		return Index >= 0 && Index < Num();
	}

	/** Return a writable reference to the value at the given Index.  Invalid to call with an index less than 0 or greater than or equal to the number of elements in the RingBuffer. */
	[[nodiscard]] ElementType& operator[](IndexType Index)
	{
		RangeCheck(Index);
		return GetAtIndexNoCheck(Index);
	}

	/** Return a const reference to the value at the given Index.  Invalid to call with an index less than 0 or greater than or equal to the number of elements in the RingBuffer. */
	[[nodiscard]] const ElementType& operator[](IndexType Index) const
	{
		RangeCheck(Index);
		return GetAtIndexNoCheck(Index);
	}

	/** Unsafely return a writable reference to the value at the given Index.  Invalid to call (and does not warn) with an index less than 0 or greater than or equal to the number of elements in the RingBuffer. */
	[[nodiscard]] ElementType& GetAtIndexNoCheck(IndexType Index)
	{
		return GetStorage()[(Front + Index) & IndexMask];
	}

	/** Unsafely return a const reference to the value at the given Index.  Invalid to call (and does not warn) with an index less than 0 or greater than or equal to the number of elements in the RingBuffer. */
	[[nodiscard]] const ElementType& GetAtIndexNoCheck(IndexType Index) const
	{
		return GetStorage()[(Front + Index) & IndexMask];
	}

	/** Given a pointer to an Element anywhere in memory, return the index of the element in the RingBuffer, or INDEX_NONE if it is not present. */
	[[nodiscard]] IndexType ConvertPointerToIndex(const ElementType* Ptr) const
	{
		const ElementType* const Data = GetStorage();
		const ElementType* const DataEnd = Data + Max();
		const ElementType* const FrontPtr = Data + (Front & IndexMask);
		IndexType Index;
		if (Ptr >= FrontPtr)
		{
			if (Ptr >= DataEnd)
			{
				return INDEX_NONE;
			}
			Index = static_cast<IndexType>(Ptr - FrontPtr);
		}
		else
		{
			if (Ptr < Data)
			{
				return INDEX_NONE;
			}
			Index = static_cast<IndexType>(Ptr - Data) + static_cast<IndexType>(DataEnd - FrontPtr);
		}
		if (Index >= Num())
		{
			return INDEX_NONE;
		}
		return Index;
	}

	/** Remove the value at the given index from the RingBuffer, and shift values ahead or behind it into its location to fill the hole.  It is valid to call with Index outside the range of the array; does nothing in that case. */
	void RemoveAt(IndexType Index)
	{
		RangeCheck(Index);
		const IndexType Capacity = Max();
		const StorageModuloType MaskedFront = Front & IndexMask;
		const StorageModuloType MaskedIndex = (Front + Index) & IndexMask;
		const StorageModuloType MaskedAfterBack = AfterBack & IndexMask;
		StorageModuloType OrderedIndex = MaskedIndex >= MaskedFront ? MaskedIndex : MaskedIndex + Capacity;
		StorageModuloType OrderedAfterBack = MaskedAfterBack >= OrderedIndex ? MaskedAfterBack : MaskedAfterBack + Capacity;
		if (OrderedIndex - MaskedFront <= OrderedAfterBack - OrderedIndex)
		{
			ShiftIndexToFront(Index);
			PopFront();
		}
		else
		{
			ShiftIndexToBack(Index);
			Pop();
		}
	}

	/**
	 * Removes as many instances of Item as there are in the array, maintaining order but not indices.
	 *
	 * @param Item Item to remove from array.
	 * @returns Number of removed elements.
	 */
	SizeType Remove(const ElementType& Item)
	{
		return RemoveAll([&Item](const ElementType& ExistingItem) { return ExistingItem == Item; });
	}

	/**
	 * Removes all items for which a given predicate applies, maintaining order but not indices.
	 *
	 * @param Predicate Any item for which the predicate returns true is removed.
	 * @returns Number of removed elements.
	 */
	template <typename PredicateType>
	SizeType RemoveAll(PredicateType Predicate)
	{
		if (AfterBack == Front)
		{
			return 0;
		}

		StorageModuloType FirstRemoval;
		ElementType* Data = GetStorage();
		for (FirstRemoval = Front; FirstRemoval != AfterBack; ++FirstRemoval)
		{
			if (Predicate(Data[FirstRemoval & IndexMask]))
			{
				break;
			}
		}
		if (FirstRemoval == AfterBack)
		{
			return 0;
		}

		StorageModuloType WriteIndex = FirstRemoval;
		StorageModuloType ReadIndex = FirstRemoval + 1;
		while (ReadIndex != AfterBack)
		{
			if (!Predicate(Data[ReadIndex & IndexMask]))
			{
				if (NeedsDestructElements())
				{
					DestructItem(&Data[WriteIndex & IndexMask]);
				}
				::new ((void*)&Data[WriteIndex & IndexMask]) ElementType(MoveTemp(Data[ReadIndex & IndexMask]));
				++WriteIndex;
			}
			++ReadIndex;
		}
		AfterBack = WriteIndex;

		SizeType NumDeleted = static_cast<SizeType>(ReadIndex - WriteIndex);
		if (NeedsDestructElements())
		{
			while (WriteIndex != ReadIndex)
			{
				DestructItem(&Data[WriteIndex & IndexMask]);
				++WriteIndex;
			}
		}
		SlackTrackerNumChanged();
		return NumDeleted;
	}

	template <typename OtherAllocator>
	[[nodiscard]] bool operator==(const TRingBuffer<ElementType, OtherAllocator>& Other) const
	{
		const IndexType LocalNum = Num();
		if (Other.Num() != LocalNum)
		{
			return false;
		}
		for (IndexType Index = 0; Index < LocalNum; ++Index)
		{
			if (!(GetAtIndexNoCheck(Index) == Other.GetAtIndexNoCheck(Index)))
			{
				return false;
			}
		}
		return true;
	}

	template <typename OtherAllocator>
	[[nodiscard]] bool operator!=(const TRingBuffer<ElementType, OtherAllocator>& Other) const
	{
		return !(*this == Other);
	}

	/**
	  * Shift all elements so that the front pointer's location in memory is less than the back pointer's. 
	  * Returns a temporary ArrayView for the RingBuffer's elements; the returned ArrayView will be invalid after the next write operation on the RingBuffer.
	  */
	TArrayView<T> Compact()
	{
		StorageModuloType MaskedFront = Front & IndexMask;
		const StorageModuloType MaskedAfterBack = AfterBack & IndexMask;
		if ((MaskedFront > MaskedAfterBack) | // Non-empty, non-full RingBuffer, and back pointer wraps around to be before front
			((MaskedFront == MaskedAfterBack) & (AfterBack != Front) & (MaskedFront != 0))) // Full, non-empty RingBuffer, and front is not at the beginning of the storage
		{
			Reallocate(Max());
			MaskedFront = Front & IndexMask;
		}
		return TArrayView<T>(GetStorage() + MaskedFront, Num());
	}

	/**
	 * Helper function to return the amount of memory allocated by this
	 * container.
	 * Only returns the size of allocations made directly by the container, not the elements themselves.
	 *
	 * @returns Number of bytes allocated by this container.
	 */
	[[nodiscard]] SIZE_T GetAllocatedSize(void) const
	{
		return Max() * sizeof(ElementType);
	}

private:
	[[nodiscard]] static constexpr bool NeedsDestructElements()
	{
		return !std::is_trivially_destructible_v<T>;
	}

	/** Set the capacity to the given value and move or copy all elements from the old storage into a new storage with the given capacity.  Assumes the capacity has already been normalized and is greater than or equal to the number of elements in the RingBuffer. */
	void Reallocate(SizeType NewCapacity)
	{
		ElementType* SrcData = GetStorage();
		const SizeType SrcCapacity = static_cast<SizeType>(Max());
		const SizeType SrcNum = static_cast<SizeType>(Num());

		check(NormalizeCapacity(NewCapacity) == NewCapacity);
		check(NewCapacity >= SrcNum);

		StorageModuloType NewIndexMask;
		if (NewCapacity > 0 && SrcNum > 0)
		{
			// Allocate SwapStorage
			ElementAllocatorType SwapStorageBuffer;
			AllocatorResizeAllocation(SwapStorageBuffer, 0, NewCapacity);
			ElementType* SwapStorage = (ElementType*)SwapStorageBuffer.GetAllocation();

			// Move data to swap storage
			const StorageModuloType MaskedFront = Front & IndexMask;
			const StorageModuloType MaskedAfterBack = AfterBack & IndexMask;

			// MaskedFront equal to MaskedAfterBack will occur if the queue's Num equals Capacity or if the Num == 0.  We checked Num == 0 above, so here it means Num == Capacity.
			if (MaskedFront >= MaskedAfterBack)
			{
				StorageModuloType WriteIndex = 0;
				for (StorageModuloType ReadIndex = MaskedFront; ReadIndex < SrcCapacity; ++ReadIndex)
				{
					::new ((void*)&SwapStorage[WriteIndex++]) ElementType(MoveTemp(SrcData[ReadIndex]));
				}
				DestructRange(MaskedFront, SrcCapacity);
				for (StorageModuloType ReadIndex = 0; ReadIndex < MaskedAfterBack; ++ReadIndex)
				{
					::new ((void*)&SwapStorage[WriteIndex++]) ElementType(MoveTemp(SrcData[ReadIndex]));
				}
				DestructRange(0, MaskedAfterBack);
			}
			else
			{
				StorageModuloType WriteIndex = 0;
				for (StorageModuloType ReadIndex = MaskedFront; ReadIndex < MaskedAfterBack; ++ReadIndex)
				{
					::new((void*)&SwapStorage[WriteIndex++]) ElementType(MoveTemp(SrcData[ReadIndex]));
				}
				DestructRange(MaskedFront, MaskedAfterBack);
			}
			// Move the swap storage into this->AllocatorInstance. "Empty" in MoveToEmpty means the elements have been
			// destructed, but the array memory might still be allocated; MoveToEmpty reallocates or frees it.
			AllocatorInstance.MoveToEmpty(SwapStorageBuffer);
			NewIndexMask = NewCapacity - 1;
		}
		else
		{
			AllocatorResizeAllocation(AllocatorInstance, 0, NewCapacity);
			NewIndexMask = static_cast<StorageModuloType>(NewCapacity - 1);
		}

		IndexMask = NewIndexMask;
		Front = 0u;
		AfterBack = SrcNum;
	}

	void AllocatorResizeAllocation(ElementAllocatorType& InAllocatorInstance, SizeType CurrentNum, SizeType NewArrayMax)
	{
		if constexpr (TAllocatorTraits<Allocator>::SupportsElementAlignment)
		{
			InAllocatorInstance.ResizeAllocation(CurrentNum, NewArrayMax, sizeof(ElementType), alignof(ElementType));
		}
		else
		{
			InAllocatorInstance.ResizeAllocation(CurrentNum, NewArrayMax, sizeof(ElementType));
		}
	}

	void SlackTrackerNumChanged()
	{
#if UE_ENABLE_ARRAY_SLACK_TRACKING
		if constexpr (TAllocatorTraits<Allocator>::SupportsSlackTracking)
		{
			AllocatorInstance.SlackTrackerLogNum(Num());
		}
#endif
	}

	/**
	 * Destruct all elements in the RingBuffer from Index RangeStart to Index RangeEnd.
	 * RangeStart and RangeEnd are given in StorageModulo space.
	 * RangeEnd - RangeStart must be less than or equal to capacity.
	 * Note that due to potential valid overflow of StorageModuleType, RangeEnd might be less than RangeStart if RangeStart is e.g. 0xffffffff and RangeEnd is 0, and RangeEnd - RangeStart is 1.
	 */
	void DestructRange(StorageModuloType RangeStart, StorageModuloType RangeEnd)
	{
		const IndexType Capacity = Max();
		if (RangeEnd - RangeStart > static_cast<StorageModuloType>(Capacity))
		{
			check(false);
			return;
		}
		const StorageModuloType DestructCount = RangeEnd - RangeStart;
		if (NeedsDestructElements() && DestructCount > 0)
		{
			ElementType* Data = GetStorage();
			const StorageModuloType MaskedRangeStart = RangeStart & IndexMask;
			const StorageModuloType MaskedRangeEnd = RangeEnd & IndexMask;
			if (MaskedRangeStart >= MaskedRangeEnd)
			{
				const SizeType FirstDestructCount = (Capacity - MaskedRangeStart);
				DestructItems(Data + MaskedRangeStart, FirstDestructCount);
				DestructItems(Data, MaskedRangeEnd);
			}
			else
			{
				DestructItems(Data + MaskedRangeStart, DestructCount);
			}
		}
	}

	/** Convert the requested capacity into the implementation-specific actual capacity that should be used. */
	SizeType NormalizeCapacity(SizeType InCapacity)
	{
		// 0 -> 0
		// All other positive integers -> RoundUpPowerOf2

		SizeType ResultCapacity = (InCapacity != 0)*FMath::RoundUpToPowerOfTwo(InCapacity);
		// Make sure we won't overflow
		// The largest value computed for a StorageModuloType is Capacity - 1 + Capacity, so we need 2*Capacity - 1 < TNumericLimits::Max, which we simplify to 2*Capacity < TNumericLimits::Max
		// We may also have overflowed InCapacity when we rounded it up, so check that InCapacity is less than ResultCapacity
		checkf(((static_cast<StorageModuloType>(ResultCapacity) <= (TNumericLimits<StorageModuloType>::Max() >> 1)) & (InCapacity <= ResultCapacity)),  TEXT("Integer overflow in TRingBuffer capacity"));
		return ResultCapacity;
	}

	/** Increase capacity if necessary to make room for the addition of a new element. */
	void ConditionalIncrementCapacity()
	{
		Reserve(Num() + 1);
	}

	/**
	 * Move the value at index RangeLast into index RangeFirst, and shift all values between RangeFirst+1 and RangeLast towards RangeLast to make room for it.
	 * RangeLast may be greater than or less than RangeFirst; the shift of the other elements will occur in the appropriate direction from RangeFirst down to RangeLast.
	 * RangeFirst and RangeLast are given in StorageModulo space.
	 * RangeDirection is -1 or 1 and indicates which direction we should move in StorageModulo space to iterate from RangeLast to RangeFirst.
	 * Required constraint: (RangeLast - RangeFirst) <= capacity for RangeDirection == -1, and (RangeFirst - RangeLast) <= capacity for RangeDirection == 1
	 */
	void ShiftLastToFirst(StorageModuloType RangeFirst, StorageModuloType RangeLast, int RangeDirection)
	{
		// This check for Range constraint is made complicated by needing to handle overflow; only subtractions of x - y can be verified, we can't use x > y because they might be on opposite sides of unsigned 0.
		// We also want to avoid doing a branch.  TODO: Is there a FMath::Select function that does this branch-less style select?
		check((RangeLast - RangeFirst) * (RangeDirection == 1) + (RangeFirst - RangeLast) * (RangeDirection != -1) <= static_cast<StorageModuloType>(Max()));

		ElementType* Data = GetStorage();
		ElementType Copy(MoveTemp(Data[RangeLast & IndexMask]));
		if (NeedsDestructElements())
		{
			for (StorageModuloType Index = RangeLast; Index != RangeFirst; Index += RangeDirection)
			{
				StorageModuloType OldValueIndex = Index & IndexMask;
				StorageModuloType NewValueIndex = (Index + RangeDirection) & IndexMask;
				DestructItems(Data + OldValueIndex, 1);
				::new ((void*)(Data + OldValueIndex)) ElementType(MoveTemp(Data[NewValueIndex]));
			}
			StorageModuloType OldValueIndex = RangeFirst & IndexMask;
			DestructItems(Data + OldValueIndex, 1);
			::new ((void*)(Data + OldValueIndex)) ElementType(MoveTemp(Copy));
		}
		else
		{
			for (StorageModuloType Index = RangeLast; Index != RangeFirst; Index += RangeDirection)
			{
				StorageModuloType OldValueIndex = Index & IndexMask;
				StorageModuloType NewValueIndex = (Index + RangeDirection) & IndexMask;
				::new ((void*)(Data + OldValueIndex)) ElementType(MoveTemp(Data[NewValueIndex]));
			}
			StorageModuloType OldValueIndex = RangeFirst & IndexMask;
			::new ((void*)(Data + OldValueIndex)) ElementType(MoveTemp(Copy));
		}
	}

	/** Return a pointer to the underlying storage of the RingBuffer */
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType* GetStorage() const
	{
		return const_cast<TRingBuffer*>(this)->GetStorage();
	}

	/** Return a const pointer to the underlying storage of the RingBuffer */
	[[nodiscard]] ElementType* GetStorage()
	{
		return (ElementType*)AllocatorInstance.GetAllocation();
	}

	/* Check and return whether the given Index is within range. */
	void RangeCheck(IndexType Index) const
	{
		if (Allocator::RequireRangeCheck) // Template property, branch will be optimized out
		{
			checkf((Index >= 0) & (Index < Num()), TEXT("RingBuffer index out of bounds: %i from a RingBuffer of size %i"), Index, Num());
		}
	}

	/* Check whether the given PopSize is within range. */
	void PopRangeCheck(SizeType PopCount) const
	{
		if (Allocator::RequireRangeCheck) // Template property, branch will be optimized out
		{
			checkf(PopCount <= static_cast<SizeType>(Num()), TEXT("RingBuffer PopCount out of bounds: %i from a RingBuffer of size %i"), PopCount, Num());
		}
	}

	friend class FRingBufferTest;

	/**
	 * The underlying storage of the RingBuffer is in the c-style array provided by this AllocatorInstance.GetAllocation().
	 * Elements in this array are uninitialized until the front pointer moves before them or the back pointer moves after them.
	 * The front pointer and the back pointer can be at arbitrary offsets in this array.
	 */
	ElementAllocatorType AllocatorInstance;
	/**
	 * A bitmask used to convert from StorageModulo space into an index into Storage.
	 * (X & IndexMask) is a valid index into GetStorage() for any value of X, as long as the RingBuffer is non-empty.
	 * Tightly tied to capacity; IndexMask == capacity - 1
	 */
	StorageModuloType IndexMask;
	/**
	 * Front pointer of the RingBuffer; it points to the first element in the iteration of the RingBuffer, unless the RingBuffer is empty, in which case it can point to any value in the underlying storage.
	 * Front is in StorageModulo space.
	 * Like all values in StorageModulo space, Front is allowed to underflow or overflow its StorageModuloType; it might be 0 and then subtract 1 to wrap around to 0xffffffff.  
	 */
	StorageModuloType Front;
	/**
	 * Pointer to the first location after the back pointer of the RingBuffer; when the RingBuffer is non-empty, one element before this index is the back of the RingBuffer
	 * AfterBack is in StorageModulo space.
	 * Like all values in StorageModulo space, AfterBack is allowed to underflow or overflow its StorageModuloType; it might be 0 and then subtract 1 to wrap around to 0xffffffff, but
	 * it is always true that (AfterBack - Front) <= capacity.
	 * Note that when AfterBack is converted from StorageModulo space into an index into underlying storage (AfterBack&IndexMask),
	 * it will be equal to (Front&IndexMask) for both an empty RingBuffer and for a RingBuffer that is at capacity.
	 */
	StorageModuloType AfterBack;
};
