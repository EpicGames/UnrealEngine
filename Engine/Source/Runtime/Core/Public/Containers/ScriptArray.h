// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Containers/AllowShrinking.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Array.h"
#include <initializer_list>


/**
 * Base dynamic array.
 * An untyped data array; mirrors a TArray's members, but doesn't need an exact C++ type for its elements.
 **/
template <typename AllocatorType>
class TScriptArray
	: protected AllocatorType::ForAnyElementType
{
public:

	[[nodiscard]] FORCEINLINE void* GetData()
	{
		return this->GetAllocation();
	}
	[[nodiscard]] FORCEINLINE const void* GetData() const
	{
		return this->GetAllocation();
	}
	[[nodiscard]] FORCEINLINE bool IsValidIndex(int32 i) const
	{
		return i>=0 && i<ArrayNum;
	}
	[[nodiscard]] bool IsEmpty() const
	{
		return ArrayNum == 0;
	}
	[[nodiscard]] FORCEINLINE int32 Num() const
	{
		checkSlow(ArrayNum>=0);
		checkSlow(ArrayMax>=ArrayNum);
		return ArrayNum;
	}
	[[nodiscard]] FORCEINLINE int32 NumUnchecked() const
	{
		return ArrayNum;
	}
	/** @return The number of elements the array can hold before reallocation. */
	[[nodiscard]] FORCEINLINE int32 Max() const
	{
		return ArrayMax;
	}
	void InsertZeroed( int32 Index, int32 Count, int32 NumBytesPerElement, uint32 AlignmentOfElement )
	{
		Insert( Index, Count, NumBytesPerElement, AlignmentOfElement );
		FMemory::Memzero( (uint8*)this->GetAllocation()+(SIZE_T)Index*NumBytesPerElement, (SIZE_T)Count*NumBytesPerElement );
	}
	void Insert( int32 Index, int32 Count, int32 NumBytesPerElement, uint32 AlignmentOfElement )
	{
		check(Count>=0);
		check(ArrayNum>=0);
		check(ArrayMax>=ArrayNum);
		check(Index>=0);
		check(Index<=ArrayNum);

		const int32 OldNum = ArrayNum;
		if( (ArrayNum+=Count)>ArrayMax )
		{
			ResizeGrow(OldNum, NumBytesPerElement, AlignmentOfElement);
		}
		FMemory::Memmove
		(
			(uint8*)this->GetAllocation() + ((SIZE_T)Index+Count )*NumBytesPerElement,
			(uint8*)this->GetAllocation() + ((SIZE_T)Index       )*NumBytesPerElement,
			((SIZE_T)OldNum-Index)*NumBytesPerElement
		);

		SlackTrackerNumChanged();
	}
	int32 Add( int32 Count, int32 NumBytesPerElement, uint32 AlignmentOfElement )
	{
		check(Count>=0);
		checkSlow(ArrayNum>=0);
		checkSlow(ArrayMax>=ArrayNum);

		const int32 OldNum = ArrayNum;
		if( (ArrayNum+=Count)>ArrayMax )
		{
			ResizeGrow(OldNum, NumBytesPerElement, AlignmentOfElement);
		}

		SlackTrackerNumChanged();

		return OldNum;
	}
	int32 AddZeroed( int32 Count, int32 NumBytesPerElement, uint32 AlignmentOfElement )
	{
		const int32 Index = Add( Count, NumBytesPerElement, AlignmentOfElement );
		FMemory::Memzero( (uint8*)this->GetAllocation()+ (SIZE_T)Index*NumBytesPerElement, (SIZE_T)Count*NumBytesPerElement );
		return Index;
	}
	void Shrink( int32 NumBytesPerElement, uint32 AlignmentOfElement )
	{
		checkSlow(ArrayNum>=0);
		checkSlow(ArrayMax>=ArrayNum);
		if (ArrayNum != ArrayMax)
		{
			ResizeTo(ArrayNum, NumBytesPerElement, AlignmentOfElement);
		}
	}
	void SetNumUninitialized(int32 NewNum, int32 NumBytesPerElement, uint32 AlignmentOfElement, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		checkSlow(NewNum >= 0);
		int32 OldNum = Num();
		if (NewNum > OldNum)
		{
			Add(NewNum - OldNum, NumBytesPerElement, AlignmentOfElement);
		}
		else if (NewNum < OldNum)
		{
			Remove(NewNum, OldNum - NewNum, NumBytesPerElement, AlignmentOfElement, AllowShrinking);
		}
	}
	void MoveAssign(TScriptArray& Other, int32 NumBytesPerElement, uint32 AlignmentOfElement)
	{
		checkSlow(this != &Other);
		Empty(0, NumBytesPerElement, AlignmentOfElement);
		this->MoveToEmpty(Other);
		ArrayNum = Other.ArrayNum; Other.ArrayNum = 0;
		ArrayMax = Other.ArrayMax; Other.ArrayMax = 0;

		this->SlackTrackerNumChanged();
		Other.SlackTrackerNumChanged();
	}
	void Empty( int32 Slack, int32 NumBytesPerElement, uint32 AlignmentOfElement )
	{
		checkSlow(Slack>=0);
		ArrayNum = 0;

		SlackTrackerNumChanged();

		if (Slack != ArrayMax)
		{
			ResizeTo(Slack, NumBytesPerElement, AlignmentOfElement);
		}
	}
	void Reset(int32 NewSize, int32 NumBytesPerElement, uint32 AlignmentOfElement)
	{
		if (NewSize <= ArrayMax)
		{
			ArrayNum = 0;

			SlackTrackerNumChanged();
		}
		else
		{
			Empty(NewSize, NumBytesPerElement, AlignmentOfElement);
		}
	}
	void SwapMemory(int32 A, int32 B, int32 NumBytesPerElement )
	{
		FMemory::Memswap(
			(uint8*)this->GetAllocation()+((SIZE_T)NumBytesPerElement*A),
			(uint8*)this->GetAllocation()+((SIZE_T)NumBytesPerElement*B),
			NumBytesPerElement
			);
	}
	TScriptArray()
		: ArrayNum( 0 )
		, ArrayMax( 0 )
	{
	}
	void CountBytes( FArchive& Ar, int32 NumBytesPerElement  ) const
	{
		Ar.CountBytes( (SIZE_T)ArrayNum*NumBytesPerElement, (SIZE_T)ArrayMax*NumBytesPerElement );
	}
	FORCEINLINE void CheckAddress(const void* Addr, int32 NumBytesPerElement) const
	{
		checkf((const char*)Addr < (const char*)GetData() || (const char*)Addr >= ((const char*)GetData() + (SIZE_T)ArrayMax * NumBytesPerElement), TEXT("Attempting to use a container element (%p) which already comes from the container being modified (%p, ArrayMax: %lld, ArrayNum: %lld, SizeofElement: %d)!"), Addr, GetData(), (long long)ArrayMax, (long long)ArrayNum, NumBytesPerElement);
	}
	/**
	 * Returns the amount of slack in this array in elements.
	 */
	[[nodiscard]] FORCEINLINE int32 GetSlack() const
	{
		return ArrayMax - ArrayNum;
	}

	void Remove(int32 Index, int32 Count, int32 NumBytesPerElement, uint32 AlignmentOfElement, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		if (Count)
		{
			checkSlow(Count >= 0);
			checkSlow(Index >= 0);
			checkSlow(Index <= ArrayNum);
			checkSlow(Index + Count <= ArrayNum);

			// Skip memmove in the common case that there is nothing to move.
			int32 NumToMove = ArrayNum - Index - Count;
			if (NumToMove)
			{
				FMemory::Memmove
					(
					(uint8*)this->GetAllocation() + ((SIZE_T)Index)* NumBytesPerElement,
					(uint8*)this->GetAllocation() + ((SIZE_T)Index + Count) * NumBytesPerElement,
					(SIZE_T)NumToMove * NumBytesPerElement
					);
			}
			ArrayNum -= Count;

			SlackTrackerNumChanged();

			if (AllowShrinking == EAllowShrinking::Yes)
			{
				ResizeShrink(NumBytesPerElement, AlignmentOfElement);
			}
			checkSlow(ArrayNum >= 0);
			checkSlow(ArrayMax >= ArrayNum);
		}
	}
	[[nodiscard]] SIZE_T GetAllocatedSize(int32 NumBytesPerElement) const
	{
		return ((const typename AllocatorType::ForAnyElementType*)this)->GetAllocatedSize(ArrayMax, NumBytesPerElement);
	}

protected:

	[[nodiscard]] TScriptArray( int32 InNum, int32 NumBytesPerElement, uint32 AlignmentOfElement )
		: ArrayNum( 0 )
		, ArrayMax( InNum )
	{
		if (ArrayMax)
		{
			ResizeInit(NumBytesPerElement, AlignmentOfElement);
		}
		ArrayNum = InNum;

		SlackTrackerNumChanged();
	}
	int32 ArrayNum;
	int32 ArrayMax;

	FORCENOINLINE void ResizeInit(int32 NumBytesPerElement, uint32 AlignmentOfElement)
	{
		ArrayMax = this->CalculateSlackReserve(ArrayMax, NumBytesPerElement, AlignmentOfElement);
		this->ResizeAllocation(ArrayNum, ArrayMax, NumBytesPerElement, AlignmentOfElement);
	}
	FORCENOINLINE void ResizeGrow(int32 OldNum, int32 NumBytesPerElement, uint32 AlignmentOfElement)
	{
		ArrayMax = this->CalculateSlackGrow(ArrayNum, ArrayMax, NumBytesPerElement, AlignmentOfElement);
		this->ResizeAllocation(OldNum, ArrayMax, NumBytesPerElement, AlignmentOfElement);
	}
	FORCENOINLINE void ResizeShrink(int32 NumBytesPerElement, uint32 AlignmentOfElement)
	{
		const int32 NewArrayMax = this->CalculateSlackShrink(ArrayNum, ArrayMax, NumBytesPerElement, AlignmentOfElement);
		if (NewArrayMax != ArrayMax)
		{
			ArrayMax = NewArrayMax;
			this->ResizeAllocation(ArrayNum, ArrayMax, NumBytesPerElement, AlignmentOfElement);
		}
	}
	FORCENOINLINE void ResizeTo(int32 NewMax, int32 NumBytesPerElement, uint32 AlignmentOfElement)
	{
		if (NewMax)
		{
			NewMax = this->CalculateSlackReserve(NewMax, NumBytesPerElement);
		}
		if (NewMax != ArrayMax)
		{
			ArrayMax = NewMax;
			this->ResizeAllocation(ArrayNum, ArrayMax, NumBytesPerElement, AlignmentOfElement);
		}
	}

private:
	FORCEINLINE void SlackTrackerNumChanged()
	{
#if UE_ENABLE_ARRAY_SLACK_TRACKING
		if constexpr (TAllocatorTraits<AllocatorType>::SupportsSlackTracking)
		{
			((typename AllocatorType::ForAnyElementType*)this)->SlackTrackerLogNum(ArrayNum);
		}
#endif
	}

public:
	/////////////////////////////////////////////////////
	// Start - intrusive TOptional<TScriptArray> state //
	/////////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TScriptArray;

	[[nodiscard]] explicit TScriptArray(FIntrusiveUnsetOptionalState Tag)
		: ArrayNum(0)
		, ArrayMax(-1)
	{
		// Use ArrayMax == -1 as our intrusive state so that the destructor still works without change, as it doesn't use ArrayMax.
	}
	[[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState Tag) const
	{
		return ArrayMax == -1;
	}
	///////////////////////////////////////////////////
	// End - intrusive TOptional<TScriptArray> state //
	///////////////////////////////////////////////////

	// These should really be private, because they shouldn't be called, but there's a bunch of code
	// that needs to be fixed first.
	TScriptArray(const TScriptArray&) { check(false); }
	void operator=(const TScriptArray&) { check(false); }
};

template<typename AllocatorType> struct TIsZeroConstructType<TScriptArray<AllocatorType>> { enum { Value = true }; };

class FScriptArray : public TScriptArray<FHeapAllocator>
{
	using Super = TScriptArray<FHeapAllocator>;

public:
	[[nodiscard]] FScriptArray() = default;

	void MoveAssign(FScriptArray& Other, int32 NumBytesPerElement, uint32 AlignmentOfElement)
	{
		Super::MoveAssign(Other, NumBytesPerElement, AlignmentOfElement);
	}

protected:
	[[nodiscard]] FScriptArray(int32 InNum, int32 NumBytesPerElement, uint32 AlignmentOfElement)
		: TScriptArray<FHeapAllocator>(InNum, NumBytesPerElement, AlignmentOfElement)
	{
	}

public:
	/////////////////////////////////////////////////////
	// Start - intrusive TOptional<FScriptArray> state //
	/////////////////////////////////////////////////////
	using IntrusiveUnsetOptionalStateType = FScriptArray;
	[[nodiscard]] explicit FScriptArray(FIntrusiveUnsetOptionalState Tag)
		: TScriptArray(Tag)
	{
	}
	///////////////////////////////////////////////////
	// End - intrusive TOptional<FScriptArray> state //
	///////////////////////////////////////////////////

	// These should really be private, because they shouldn't be called, but there's a bunch of code
	// that needs to be fixed first.
	[[nodiscard]] FScriptArray(const FScriptArray&) { check(false); }
	void operator=(const FScriptArray&) { check(false); }
};

template<> struct TIsZeroConstructType<FScriptArray> { enum { Value = true }; };
