// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContainersFwd.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/CompactSet.h"
#include "Containers/SetUtilities.h"
#include "Templates/Function.h"

using FScriptCompactSetLayout = FCompactSetLayout;

// Untyped set type for accessing TCompactSet data, like FScriptArray for TArray.
// Must have the same memory representation as a TCompactSet.
template <typename Allocator>
class TScriptCompactSet : public TCompactSetBase<typename Allocator::template ElementAllocator<sizeof(uint8)>>
{
	using Super = TCompactSetBase<typename Allocator::template ElementAllocator<sizeof(uint8)>>;
public:
	[[nodiscard]] static constexpr FScriptCompactSetLayout GetScriptLayout(int32 ElementSize, int32 ElementAlignment)
	{
		return { ElementSize, FGenericPlatformMath::Max<int32>(ElementAlignment, UE::Core::CompactHashTable::GetMemoryAlignment()) };
	}

	//////////////////////////////////////////////////////////
	// Start - intrusive TOptional<TScriptCompactSet> state //
	//////////////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TScriptCompactSet;

	[[nodiscard]] explicit TScriptCompactSet(FIntrusiveUnsetOptionalState Tag) : Super(Tag)
	{
	}
	////////////////////////////////////////////////////////
	// End - intrusive TOptional<TScriptCompactSet> state //
	////////////////////////////////////////////////////////

	[[nodiscard]] constexpr TScriptCompactSet() = default;

	[[nodiscard]] bool IsValidIndex(int32 Index) const
	{
		return Index >= 0 && Index < this->NumElements;
	}

	[[nodiscard]] int32 NumUnchecked() const
	{
		return this->NumElements;
	}

	[[nodiscard]] void* GetData(int32 Index, const FScriptCompactSetLayout& Layout)
	{
		return (uint8*)this->Elements.GetAllocation() + (Layout.Size * Index);
	}

	[[nodiscard]] const void* GetData(int32 Index, const FScriptCompactSetLayout& Layout) const
	{
		return (const uint8 *)this->Elements.GetAllocation() + (Layout.Size * Index);
	}

	void MoveAssign(TScriptCompactSet& Other, const FScriptCompactSetLayout& Layout)
	{
		checkSlow(this != &Other);

		this->Elements.MoveToEmpty(Other.Elements);
		this->NumElements = Other.NumElements;
		this->MaxElements = Other.MaxElements;

		Other.NumElements = 0;
		Other.MaxElements = 0;
	}

	void Empty(int32 Slack, const FScriptCompactSetLayout& Layout)
	{
		this->ResizeAllocation(Slack, Layout);
		if (this->MaxElements)
		{
			this->GetHashTableView(Layout).Reset();
		}
		this->NumElements = 0;
	}

	void RemoveAt(int32 Index, const FScriptCompactSetLayout& Layout, TFunctionRef<uint32 (const void*)> GetKeyHash, TFunctionRef<void (void*)> DestructItem)
	{
		check(IsValidIndex(Index));

		FCompactHashTableView HashTable = this->GetHashTableView(Layout);

		void* Dst = GetData(Index, Layout);
#if UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE
		const uint32 DstKeyHash = HashTable.GetStoredKey(Index, this->NumElements);
#else
		const uint32 DstKeyHash = GetKeyHash(Dst);
#endif

		const int32 LastElementIndex = this->NumElements - 1;
		if (Index == LastElementIndex)
		{
			HashTable.Remove(Index, DstKeyHash, Index, 0);
			DestructItem(Dst);
		}
		else
		{
			void* Src = GetData(LastElementIndex, Layout);
#if UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE
			const uint32 SrcKeyHash = HashTable.GetStoredKey(LastElementIndex, this->NumElements);
#else
			const uint32 SrcKeyHash = GetKeyHash(Src);
#endif
			HashTable.Remove(Index, DstKeyHash, LastElementIndex, SrcKeyHash);
			DestructItem(Dst);

			// Memmove is fine here as our containers already only work if data is trivially relocatable
			FMemory::Memmove(Dst, Src, Layout.Size);
		}

		--this->NumElements;
	}

	/**
	 * Adds an uninitialized object to the set.
	 * The set will need rehashing at some point after this call to make it valid.
	 *
	 * @return  The index of the added element.
	 */
	int32 AddUninitialized(const FScriptCompactSetLayout& Layout)
	{
		checkSlow(this->NumElements >= 0 && this->MaxElements >= INDEX_NONE);
		if (this->NumElements == this->MaxElements)
		{
			this->ResizeAllocation(this->AllocatorCalculateSlackGrow(this->NumElements + 1, Layout), Layout);
		}
		++this->NumElements;
		return this->NumElements - 1;
	}

	void RemoveAtUninitialized(const FScriptCompactSetLayout& Layout, int32 Index)
	{
		// Can only be pairs with AddUninitialized for now
		check(Index == this->NumElements - 1);
		--this->NumElements;
	}

	void CommitLastUninitialized(const FScriptCompactSetLayout& Layout, TFunctionRef<uint32 (const void*)> GetKeyHash)
	{
		const uint32 KeyHash = GetKeyHash(GetData(this->NumElements - 1, Layout));
		this->GetHashTableView(Layout).Add(this->NumElements - 1, KeyHash);
	}

	void CommitAllUninitialized(const FScriptCompactSetLayout& Layout, TFunctionRef<uint32 (const void*)> GetKeyHash)
	{
		// Keep consistent interface with ScriptSparseSet
	}

	void Rehash(const FScriptCompactSetLayout& Layout, TFunctionRef<uint32 (const void*)> GetKeyHash)
	{
		if (this->MaxElements > 0)
		{
			const uint8* ElementData = this->Elements.GetAllocation();
			FCompactHashTableView HashTable = this->GetHashTableView(Layout);

			HashTable.Reset();

			for (int32 Index = 0; Index < this->NumElements; ++Index)
			{
				HashTable.Add(Index, GetKeyHash(ElementData + Layout.Size * Index));
			}
		}
	}

	[[nodiscard]] int32 FindIndex(const void* Element, const FScriptCompactSetLayout& Layout, TFunctionRef<uint32 (const void*)> GetKeyHash, TFunctionRef<bool (const void*, const void*)> EqualityFn) const
	{
		if (this->NumElements)
		{
			return FindIndexImpl(Element, Layout, GetKeyHash(Element), EqualityFn);
		}

		return INDEX_NONE;
	}

	[[nodiscard]] int32 FindIndexByHash(const void* Element, const FScriptCompactSetLayout& Layout, uint32 KeyHash, TFunctionRef<bool (const void*, const void*)> EqualityFn) const
	{
		if (this->NumElements)
		{
			return FindIndexImpl(Element, Layout, KeyHash, EqualityFn);
		}

		return INDEX_NONE;
	}

	int32 FindOrAdd(const void* Element, const FScriptCompactSetLayout& Layout, TFunctionRef<uint32(const void*)> GetKeyHash, TFunctionRef<bool(const void*, const void*)> EqualityFn, TFunctionRef<void(void*)> ConstructFn)
	{
		uint32 KeyHash = GetKeyHash(Element);
		int32 OldElementIndex = FindIndexByHash(Element, Layout, KeyHash, EqualityFn);
		if (OldElementIndex != INDEX_NONE)
		{
			return OldElementIndex;
		}

		return AddNewElement(Layout, GetKeyHash, KeyHash, ConstructFn);
	}

	void Add(const void* Element, const FScriptCompactSetLayout& Layout, TFunctionRef<uint32(const void*)> GetKeyHash, TFunctionRef<bool(const void*, const void*)> EqualityFn, TFunctionRef<void(void*)> ConstructFn, TFunctionRef<void(void*)> DestructFn)
	{
		uint32 KeyHash = GetKeyHash(Element);
		int32 OldElementIndex = FindIndexByHash(Element, Layout, KeyHash, EqualityFn);
		if (OldElementIndex != INDEX_NONE)
		{
			void* ElementPtr = GetData(OldElementIndex, Layout);

			DestructFn(ElementPtr);
			ConstructFn(ElementPtr);

			// We don't update the hash because we don't need to - the new element
			// should have the same hash, but let's just check.
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// Disable deprecations warnings to stop warnings being thrown by our check macro.
			checkSlow(KeyHash == GetKeyHash(ElementPtr));
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else
		{
			AddNewElement(Layout, GetKeyHash, KeyHash, ConstructFn);
		}
	}

private:
	[[nodiscard]] int32 FindIndexImpl(const void* Element, const FScriptCompactSetLayout& Layout, uint32 KeyHash, TFunctionRef<bool (const void*, const void*)> EqualityFn) const
	{
		return this->GetConstHashTableView(Layout).Find(KeyHash, this->NumElements, [Element, &Layout, &EqualityFn, this](uint32 Index) { return EqualityFn(Element, GetData(Index, Layout)); });
	}

	int32 AddNewElement(const FScriptCompactSetLayout& Layout, TFunctionRef<uint32(const void*)> GetKeyHash, uint32 KeyHash, TFunctionRef<void(void*)> ConstructFn)
	{
		checkSlow(this->NumElements >= 0 && this->MaxElements >= 0);
		if (this->NumElements == this->MaxElements)
		{
			if (this->ResizeAllocationPreserveData(this->AllocatorCalculateSlackGrow(this->NumElements + 1, Layout), Layout))
			{
				Rehash(Layout, GetKeyHash);
			}
		}

		this->GetHashTableView(Layout).Add(this->NumElements, KeyHash);

		ConstructFn(GetData(this->NumElements, Layout));
		return this->NumElements++;
	}

public:
	// These should really be private, because they shouldn't be called, but there's a bunch of code
	// that needs to be fixed first.
	TScriptCompactSet(const TScriptCompactSet&) { check(false); }
	void operator=(const TScriptCompactSet&) { check(false); }
};

template <typename AllocatorType>
struct TIsZeroConstructType<TScriptCompactSet<AllocatorType>>
{
	enum { Value = true };
};

using FScriptCompactSet = TScriptCompactSet<FDefaultCompactSetAllocator>;
