// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContainersFwd.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/SetUtilities.h"
#include "Containers/SparseSet.h"
#include "Misc/StructBuilder.h"
#include "Templates/Function.h"

struct FScriptSparseSetLayout
{
	// int32 ElementOffset = 0; // always at zero offset from the TSparseSetElement - not stored here
	int32 HashNextIdOffset;
	int32 HashIndexOffset;
	int32 Size;

	FScriptSparseArrayLayout SparseArrayLayout;
};

// Untyped set type for accessing TSparseSet data, like FScriptArray for TArray.
// Must have the same memory representation as a TSparseSet.
template <typename Allocator>
class TScriptSparseSet
{
public:
	[[nodiscard]] static constexpr FScriptSparseSetLayout GetScriptLayout(int32 ElementSize, int32 ElementAlignment)
	{
		FScriptSparseSetLayout Result;

		// TSparseSetElement<TPair<Key, Value>>
		FStructBuilder SetElementStruct;
		int32 ElementOffset      = SetElementStruct.AddMember(ElementSize,           ElementAlignment);
		Result.HashNextIdOffset  = SetElementStruct.AddMember(sizeof(FSetElementId), alignof(FSetElementId));
		Result.HashIndexOffset   = SetElementStruct.AddMember(sizeof(int32),         alignof(int32));
		Result.Size              = SetElementStruct.GetSize();
		Result.SparseArrayLayout = FScriptSparseArray::GetScriptLayout(SetElementStruct.GetSize(), SetElementStruct.GetAlignment());

		checkf(ElementOffset == 0, TEXT("The element inside the TSparseSetElement is expected to be at the start of the struct"));

		return Result;
	}

	[[nodiscard]] TScriptSparseSet()
		: HashSize(0)
	{
	}

	///////////////////////////////////////////////////
	// Start - intrusive TOptional<TScriptSparseSet> state //
	///////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TScriptSparseSet;

	[[nodiscard]] explicit TScriptSparseSet(FIntrusiveUnsetOptionalState Tag)
	: Elements(Tag)
	{
	}
	[[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState Tag) const
	{
		return Elements == Tag;
	}
	/////////////////////////////////////////////////
	// End - intrusive TOptional<TScriptSparseSet> state //
	/////////////////////////////////////////////////

	[[nodiscard]] bool IsValidIndex(int32 Index) const
	{
		return Elements.IsValidIndex(Index);
	}

	[[nodiscard]] bool IsEmpty() const
	{
		return Elements.IsEmpty();
	}

	[[nodiscard]] bool IsCompact() const
	{
		return Elements.IsCompact();
	}

	[[nodiscard]] int32 Num() const
	{
		return Elements.Num();
	}

	[[nodiscard]] int32 NumUnchecked() const
	{
		return Elements.NumUnchecked();
	}

	/** @return The number of elements the set can hold before reallocation. */
	[[nodiscard]] int32 Max() const
	{
		return Elements.Max();
	}

	[[nodiscard]] int32 GetMaxIndex() const
	{
		return Elements.GetMaxIndex();
	}

	[[nodiscard]] void* GetData(int32 Index, const FScriptSparseSetLayout& Layout)
	{
		return Elements.GetData(Index, Layout.SparseArrayLayout);
	}

	[[nodiscard]] const void* GetData(int32 Index, const FScriptSparseSetLayout& Layout) const
	{
		return Elements.GetData(Index, Layout.SparseArrayLayout);
	}

	void MoveAssign(TScriptSparseSet& Other, const FScriptSparseSetLayout& Layout)
	{
		checkSlow(this != &Other);
		Empty(0, Layout);
		Elements.MoveAssign(Other.Elements, Layout.SparseArrayLayout);
		Hash.MoveToEmpty(Other.Hash);
		HashSize = Other.HashSize; Other.HashSize = 0;
	}

	void Empty(int32 Slack, const FScriptSparseSetLayout& Layout)
	{
		// Empty the elements array, and reallocate it for the expected number of elements.
		Elements.Empty(Slack, Layout.SparseArrayLayout);

		// Calculate the desired hash size for the specified number of elements.
		const int32 DesiredHashSize = Allocator::GetNumberOfHashBuckets(Slack);

		// If the hash hasn't been created yet, or is smaller than the desired hash size, rehash.
		if (Slack != 0 && (HashSize == 0 || HashSize != DesiredHashSize))
		{
			HashSize = DesiredHashSize;

			// Free the old hash.
			Hash.ResizeAllocation(0, HashSize, sizeof(FSetElementId));
		}

		FSetElementId* HashPtr = Hash.GetAllocation();
		for (int32 I = 0; I < HashSize; ++I)
		{
			HashPtr[I] = FSetElementId();
		}
	}

	void RemoveAt(int32 Index, const FScriptSparseSetLayout& Layout)
	{
		check(IsValidIndex(Index));

		void* ElementBeingRemoved = Elements.GetData(Index, Layout.SparseArrayLayout);

		// Remove the element from the hash.
		for (FSetElementId* NextElementId = &GetTypedHash(GetHashIndexRef(ElementBeingRemoved, Layout)); NextElementId->IsValidId(); NextElementId = &GetHashNextIdRef(Elements.GetData(NextElementId->AsInteger(), Layout.SparseArrayLayout), Layout))
		{
			if (NextElementId->AsInteger() == Index)
			{
				*NextElementId = GetHashNextIdRef(ElementBeingRemoved, Layout);
				break;
			}
		}

		// Remove the element from the elements array.
		Elements.RemoveAtUninitialized(Layout.SparseArrayLayout, Index);
	}

	/**
	 * Adds an uninitialized object to the set.
	 * The set will need rehashing at some point after this call to make it valid.
	 *
	 * @return  The index of the added element.
	 */
	int32 AddUninitialized(const FScriptSparseSetLayout& Layout)
	{
		return Elements.AddUninitialized(Layout.SparseArrayLayout);
	}

	void RemoveAtUninitialized(const FScriptSparseSetLayout& Layout, int32 Index)
	{
		Elements.RemoveAtUninitialized(Layout.SparseArrayLayout, Index);
	}

	void CommitLastUninitialized(const FScriptSparseSetLayout& Layout, TFunctionRef<uint32 (const void*)> GetKeyHash)
	{
		// Keep consistent interface with ScriptCompactSet
	}

	void CommitAllUninitialized(const FScriptSparseSetLayout& Layout, TFunctionRef<uint32 (const void*)> GetKeyHash)
	{
		Rehash(Layout, GetKeyHash);
	}

	void Rehash(const FScriptSparseSetLayout& Layout, TFunctionRef<uint32 (const void*)> GetKeyHash)
	{
		// Free the old hash.
		Hash.ResizeAllocation(0,0,sizeof(FSetElementId));

		HashSize = Allocator::GetNumberOfHashBuckets(Elements.Num());
		if (HashSize)
		{
			// Allocate the new hash.
			checkSlow(FMath::IsPowerOfTwo(HashSize));
			Hash.ResizeAllocation(0, HashSize, sizeof(FSetElementId));
			for (int32 HashIndex = 0; HashIndex < HashSize; ++HashIndex)
			{
				GetTypedHash(HashIndex) = FSetElementId();
			}

			// Add the existing elements to the new hash.
			int32 Index = 0;
			int32 Count = Elements.Num();
			while (Count)
			{
				if (Elements.IsValidIndex(Index))
				{
					FSetElementId ElementId = FSetElementId::FromInteger(Index);

					void* Element = (uint8*)Elements.GetData(Index, Layout.SparseArrayLayout);

					// Compute the hash bucket the element goes in.
					uint32 KeyHash = GetKeyHash(Element);
					int32  HashIndex   = KeyHash & (HashSize - 1);
					GetHashIndexRef(Element, Layout) = KeyHash & (HashSize - 1);

					// Link the element into the hash bucket.
					GetHashNextIdRef(Element, Layout) = GetTypedHash(HashIndex);
					GetTypedHash(HashIndex) = ElementId;

					--Count;
				}

				++Index;
			}
		}
	}

private:
	[[nodiscard]] int32 FindIndexImpl(const void* Element, const FScriptSparseSetLayout& Layout, uint32 KeyHash, TFunctionRef<bool (const void*, const void*)> EqualityFn) const
	{
		const int32  HashIndex = KeyHash & (HashSize - 1);

		uint8* CurrentElement = nullptr;
		for (FSetElementId ElementId = GetTypedHash(HashIndex);
			ElementId.IsValidId();
			ElementId = GetHashNextIdRef(CurrentElement, Layout))
		{
			CurrentElement = (uint8*)Elements.GetData(ElementId.AsInteger(), Layout.SparseArrayLayout);
			if (EqualityFn(Element, CurrentElement))
			{
				return ElementId.AsInteger();
			}
		}

		return INDEX_NONE;
	}

public:
	[[nodiscard]] int32 FindIndex(const void* Element, const FScriptSparseSetLayout& Layout, TFunctionRef<uint32 (const void*)> GetKeyHash, TFunctionRef<bool (const void*, const void*)> EqualityFn) const
	{
		if (Elements.Num())
		{
			return FindIndexImpl(Element, Layout, GetKeyHash(Element), EqualityFn);
		}

		return INDEX_NONE;
	}


	[[nodiscard]] int32 FindIndexByHash(const void* Element, const FScriptSparseSetLayout& Layout, uint32 KeyHash, TFunctionRef<bool (const void*, const void*)> EqualityFn) const
	{
		if (Elements.Num())
		{
			return FindIndexImpl(Element, Layout, KeyHash, EqualityFn);
		}

		return INDEX_NONE;
	}

	int32 FindOrAdd(const void* Element, const FScriptSparseSetLayout& Layout, TFunctionRef<uint32(const void*)> GetKeyHash, TFunctionRef<bool(const void*, const void*)> EqualityFn, TFunctionRef<void(void*)> ConstructFn)
	{
		uint32 KeyHash = GetKeyHash(Element);
		int32 OldElementIndex = FindIndexByHash(Element, Layout, KeyHash, EqualityFn);
		if (OldElementIndex != INDEX_NONE)
		{
			return OldElementIndex;
		}

		return AddNewElement(Layout, GetKeyHash, KeyHash, ConstructFn);
	}

	void Add(const void* Element, const FScriptSparseSetLayout& Layout, TFunctionRef<uint32(const void*)> GetKeyHash, TFunctionRef<bool(const void*, const void*)> EqualityFn, TFunctionRef<void(void*)> ConstructFn, TFunctionRef<void(void*)> DestructFn)
	{
		uint32 KeyHash = GetKeyHash(Element);
		int32 OldElementIndex = FindIndexByHash(Element, Layout, KeyHash, EqualityFn);
		if (OldElementIndex != INDEX_NONE)
		{
			void* ElementPtr = Elements.GetData(OldElementIndex, Layout.SparseArrayLayout);

			DestructFn(ElementPtr);
			ConstructFn(ElementPtr);

			// We don't update the hash because we don't need to - the new element
			// should have the same hash, but let's just check.
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// Disable deprecations warnings to stop warnings being thrown by our check macro.
			checkSlow(KeyHash == GetKeyHash(ElementPtr));
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else
		{
			AddNewElement(Layout, GetKeyHash, KeyHash, ConstructFn);
		}
	}

private:
	int32 AddNewElement(const FScriptSparseSetLayout& Layout, TFunctionRef<uint32(const void*)> GetKeyHash, uint32 KeyHash, TFunctionRef<void(void*)> ConstructFn)
	{
		int32 NewElementIndex = Elements.AddUninitialized(Layout.SparseArrayLayout);

		void* ElementPtr = Elements.GetData(NewElementIndex, Layout.SparseArrayLayout);
		ConstructFn(ElementPtr);

		const int32 DesiredHashSize = FDefaultSparseSetAllocator::GetNumberOfHashBuckets(Num());
		if (!HashSize || HashSize < DesiredHashSize)
		{
			// rehash, this will link in our new element if needed:
			Rehash(Layout, GetKeyHash);
		}
		else
		{
			// link the new element into the set:
			int32 HashIndex = KeyHash & (HashSize - 1);
			FSetElementId& TypedHash = GetTypedHash(HashIndex);
			GetHashIndexRef(ElementPtr, Layout) = HashIndex;
			GetHashNextIdRef(ElementPtr, Layout) = TypedHash;
			TypedHash = FSetElementId::FromInteger(NewElementIndex);
		}

		return NewElementIndex;
	}

	/** Encapsulates the allocators used by a sparse array in a single type. */
	class TrackedSparseArrayAllocator
	{
	public:

		using ElementAllocator = typename Allocator::SparseArrayAllocator::ElementAllocator;
		using BitArrayAllocator = typename Allocator::SparseArrayAllocator::BitArrayAllocator;
	};

	using HashAllocator = typename Allocator::HashAllocator;

	using ElementArrayType = TScriptSparseArray<TrackedSparseArrayAllocator>;
	using HashType         = typename HashAllocator::template ForElementType<FSetElementId>;

	ElementArrayType Elements;
	HashType         Hash;
	int32            HashSize;

	[[nodiscard]] UE_FORCEINLINE_HINT FSetElementId& GetTypedHash(int32 HashIndex) const
	{
		return ((FSetElementId*)Hash.GetAllocation())[HashIndex & (HashSize - 1)];
	}

	[[nodiscard]] static FSetElementId& GetHashNextIdRef(const void* Element, const FScriptSparseSetLayout& Layout)
	{
		return *(FSetElementId*)((uint8*)Element + Layout.HashNextIdOffset);
	}

	[[nodiscard]] static int32& GetHashIndexRef(const void* Element, const FScriptSparseSetLayout& Layout)
	{
		return *(int32*)((uint8*)Element + Layout.HashIndexOffset);
	}

	// This function isn't intended to be called, just to be compiled to validate the correctness of the type.
	static void CheckConstraints()
	{
		typedef TScriptSparseSet  ScriptType;
		typedef TSparseSet<int32> RealType;

		// Check that the class footprint is the same
		static_assert(sizeof (ScriptType) == sizeof (RealType), "TScriptSparseSet's size doesn't match TSparseSet");
		static_assert(alignof(ScriptType) == alignof(RealType), "TScriptSparseSet's alignment doesn't match TSparseSet");

		// Check member sizes
		static_assert(sizeof(DeclVal<ScriptType>().Elements) == sizeof(DeclVal<RealType>().Elements), "TScriptSparseSet's Elements member size does not match TSparseSet's");
		static_assert(sizeof(DeclVal<ScriptType>().Hash)     == sizeof(DeclVal<RealType>().Hash),     "TScriptSparseSet's Hash member size does not match TSparseSet's");
		static_assert(sizeof(DeclVal<ScriptType>().HashSize) == sizeof(DeclVal<RealType>().HashSize), "TScriptSparseSet's HashSize member size does not match TSparseSet's");

		// Check member offsets
		static_assert(STRUCT_OFFSET(ScriptType, Elements) == STRUCT_OFFSET(RealType, Elements), "TScriptSparseSet's Elements member offset does not match TSparseSet's");
		static_assert(STRUCT_OFFSET(ScriptType, Hash)     == STRUCT_OFFSET(RealType, Hash),     "TScriptSparseSet's Hash member offset does not match TSparseSet's");
		static_assert(STRUCT_OFFSET(ScriptType, HashSize) == STRUCT_OFFSET(RealType, HashSize), "TScriptSparseSet's FirstFreeIndex member offset does not match TSparseSet's");
	}

public:
	// These should really be private, because they shouldn't be called, but there's a bunch of code
	// that needs to be fixed first.
	TScriptSparseSet(const TScriptSparseSet&) { check(false); }
	void operator=(const TScriptSparseSet&) { check(false); }
};

template <typename AllocatorType>
struct TIsZeroConstructType<TScriptSparseSet<AllocatorType>>
{
	enum { Value = true };
};
