// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainerAllocationPolicies.h"

// Note: We're using define injection to define TSet instead of a using alias as the current MSVC compiler (up to at least 19.44)
// is issuing an error due to the use of a deduction guide in TCompactSet/TSparseSet when using the form TSet(TArray<>).
// If this gets resolve we should revert to a using alias for clarity, reduced header includes, potential
// interoperation between related types and to help intellisense.

#if UE_USE_COMPACT_SET_AS_DEFAULT
#include "Containers/ScriptCompactSet.h"

using FScriptSetLayout = FScriptCompactSetLayout;

template<typename Allocator>
using TScriptSet = TScriptCompactSet<Allocator>;

#define UE_TCOMPACT_SET TSet
#include "Containers/CompactSet.h.inl"
#undef UE_TCOMPACT_SET

// Todo: This should be removed when existing code is fixed to explicitly include what it uses
#include "Containers/BitArray.h"

#else

#include "Containers/ScriptSparseSet.h"

#define UE_TSPARSE_SET TSet
#include "Containers/SparseSet.h.inl"
#undef UE_TSPARSE_SET

using FScriptSetLayout = FScriptSparseSetLayout;

template<typename Allocator>
using TScriptSet = TScriptSparseSet<Allocator>;

#endif

using FScriptSet = TScriptSet<FDefaultSetAllocator>;
