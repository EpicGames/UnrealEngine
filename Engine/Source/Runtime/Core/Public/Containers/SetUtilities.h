// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Serialization/MemoryLayout.h"
#include "Templates/MemoryOps.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTypeTraits.h"

// These divisors will determine at what point it's more optimal to individually remove elements from the hash table vs just using memset to reset the whole thing at once
// Sparse set stores the hashtable index so it's fairly cheap to remove vs compact set that has to recalculate the hash key which is more expensive
#define UE_SET_UNHASH_THRESHOLD_STORED_KEY 4
#define UE_SET_UNHASH_THRESHOLD_CALCULATE_KEY 32

/**
 * Traits which determine whether or not a type is a TSet.
 */
template <typename T>
inline constexpr bool TIsTSet_V = false;

template <typename T> inline constexpr bool TIsTSet_V<const          T> = TIsTSet_V<T>;
template <typename T> inline constexpr bool TIsTSet_V<      volatile T> = TIsTSet_V<T>;
template <typename T> inline constexpr bool TIsTSet_V<const volatile T> = TIsTSet_V<T>;

template <typename T>
struct TIsTSet
{
	static inline constexpr bool Value = TIsTSet_V<T>;
};

/**
 * The base KeyFuncs type with some useful definitions for all KeyFuncs; meant to be derived from instead of used directly.
 * bInAllowDuplicateKeys=true is slightly faster because it allows the TSet to skip validating that
 * there isn't already a duplicate entry in the TSet.
  */
template<typename ElementType,typename InKeyType,bool bInAllowDuplicateKeys = false>
struct BaseKeyFuncs
{
	typedef InKeyType KeyType;
	typedef typename TCallTraits<InKeyType>::ParamType KeyInitType;
	typedef typename TCallTraits<ElementType>::ParamType ElementInitType;

	static inline constexpr bool bAllowDuplicateKeys = bInAllowDuplicateKeys;
};

/**
 * A default implementation of the KeyFuncs used by TSet which uses the element as a key.
 */
template<typename ElementType,bool bInAllowDuplicateKeys /*= false*/>
struct DefaultKeyFuncs : BaseKeyFuncs<ElementType,ElementType,bInAllowDuplicateKeys>
{
	typedef typename TTypeTraits<ElementType>::ConstPointerType KeyInitType;
	typedef typename TCallTraits<ElementType>::ParamType ElementInitType;

	/**
	 * @return The key used to index the given element.
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT KeyInitType GetSetKey(ElementInitType Element)
	{
		return Element;
	}

	/**
	 * @return True if the keys match.
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT bool Matches(KeyInitType A, KeyInitType B)
	{
		return A == B;
	}

	/**
	 * @return True if the keys match.
	 */
	template<typename ComparableKey>
	[[nodiscard]] static UE_FORCEINLINE_HINT bool Matches(KeyInitType A, ComparableKey B)
	{
		return A == B;
	}

	/** Calculates a hash index for a key. */
	[[nodiscard]] static UE_FORCEINLINE_HINT uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key);
	}

	/** Calculates a hash index for a key. */
	template<typename ComparableKey>
	[[nodiscard]] static UE_FORCEINLINE_HINT uint32 GetKeyHash(ComparableKey Key)
	{
		return GetTypeHash(Key);
	}
};

/** This is used to provide type specific behavior for a move which will destroy B. */
/** Should be in UnrealTemplate but isn't for Clang build reasons - will move later */
template<typename T>
inline void MoveByRelocate(T& A, T& B)
{
	// Destruct the previous value of A.
	A.~T();

	// Relocate B into the 'hole' left by the destruction of A, leaving a hole in B instead.
	RelocateConstructItems<T>(&A, &B, 1);
}

/** Either NULL or an identifier for an element of a set.
 * Used to differentiate between int as an element type and an index to a specific location
 */
class FSetElementId
{
public:
	/** Default constructor. */
	constexpr UE_FORCEINLINE_HINT FSetElementId() = default;

	/** @return a boolean value representing whether the id is NULL. */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValidId() const
	{
		return Index != INDEX_NONE;
	}

	/** Comparison operators. */
	[[nodiscard]] UE_FORCEINLINE_HINT friend bool operator==(const FSetElementId& A, const FSetElementId& B)
	{
		return A.Index == B.Index;
	}
	[[nodiscard]] UE_FORCEINLINE_HINT friend bool operator!=(const FSetElementId& A, const FSetElementId& B)
	{
		return A.Index != B.Index;
	}

	[[nodiscard]] constexpr UE_FORCEINLINE_HINT int32 AsInteger() const
	{
		return Index;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT static FSetElementId FromInteger(int32 Integer)
	{
		return FSetElementId(Integer);
	}

private:
	/** The index of the element in the set's element array. */
	int32 Index = INDEX_NONE;

	/** Initialization constructor. */
	[[nodiscard]] UE_FORCEINLINE_HINT explicit FSetElementId(int32 InIndex)
	: Index(InIndex)
	{
	}
};

// This is just an int32
DECLARE_INTRINSIC_TYPE_LAYOUT(FSetElementId);
