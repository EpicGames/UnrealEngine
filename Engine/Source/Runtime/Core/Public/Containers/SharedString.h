// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AtomicRefCount.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "CoreTypes.h"
#include "HAL/MemoryBase.h"
#include "HAL/UnrealMemory.h"
#include "Misc/CString.h"
#include "Misc/UEOps.h"

#include <atomic>

namespace UE
{

/**
 * A reference to an immutable, shared, reference-counted string.
 *
 * Prefer TStringView when there is a clear single point of ownership with a longer lifetime than
 * the references to the string. This type is meant for cases where a shared string does not have
 * an obvious owner or where the lifetime is not easy to manage.
 *
 * The string is stored as a pointer to the start of the null-terminated string which is preceded
 * by a 4-byte reference count and a 4-byte size.
 */
template <typename CharType>
class TSharedString
{
public:
	using ElementType = CharType;

	[[nodiscard]] TSharedString() = default;
	[[nodiscard]] TSharedString(TSharedString&& String);
	[[nodiscard]] TSharedString(const TSharedString& String);
	TSharedString& operator=(TSharedString&& String);
	TSharedString& operator=(const TSharedString& String);

	/** Allocates a copy of the string and constructs this as a reference to it. */
	[[nodiscard]] TSharedString(TStringView<CharType> String);
	[[nodiscard]] UE_REWRITE TSharedString(const CharType* String)
		: TSharedString(TStringView<CharType>(String)) 
	{
	}

	/** Allocates a copy of the string and assigns this as a reference to it. */
	TSharedString& operator=(TStringView<CharType> String);
	UE_REWRITE TSharedString& operator=(const CharType* String)
	{ 
		return operator=(TStringView<CharType>(String)); 
	}

	inline ~TSharedString()
	{
		Release(Chars);
	}

	/** Resets this to reference the empty string. */
	inline void Reset()
	{
		Release(Chars);
		Chars = nullptr;
	}

	/** Returns true if the referenced string is empty. */
	[[nodiscard]] inline bool IsEmpty() const
	{
		return !Chars;
	}

	/** Returns the length of the referenced string excluding the null terminator. */
	[[nodiscard]] inline int32 Len() const
	{
		return Chars ? reinterpret_cast<const int32*>(Chars)[-1] : 0;
	}

	/** Returns a pointer to the start of the referenced null-terminated string. */
	[[nodiscard]] inline const CharType* operator*() const
	{
		return Chars ? Chars : &NullChar;
	}

	/** Returns the amount of allocated bytes for this TSharedString. */
	inline SIZE_T GetAllocatedSize() const
	{
		return sizeof(int32) + sizeof(int32) + sizeof(CharType) + sizeof(CharType) * Len();
	}

	/** Returns a TStringView on this TSharedString. */
	inline TStringView<CharType> ToView() const 
	{
		return {operator*(), Len()};
	}

	/** An empty string provided mainly for returning a reference to an empty TSharedString. */
	static const TSharedString Empty;

	[[nodiscard]] inline bool UEOpEquals(const TSharedString& Rhs) const
	{
		return MakeStringView(*this).Equals(MakeStringView(Rhs), ESearchCase::IgnoreCase);
	}
	[[nodiscard]] inline bool UEOpEquals(const CharType* Rhs) const
	{
		return MakeStringView(*this).Equals(Rhs, ESearchCase::IgnoreCase);
	}
	[[nodiscard]] inline bool UEOpLessThan(const TSharedString& Rhs) const
	{
		return MakeStringView(*this).Compare(MakeStringView(Rhs), ESearchCase::IgnoreCase) < 0;
	}
	[[nodiscard]] inline bool UEOpLessThan(const CharType* Rhs) const
	{
		return MakeStringView(*this).Compare(Rhs, ESearchCase::IgnoreCase) < 0;
	}
	[[nodiscard]] inline bool UEOpGreaterThan(const CharType* Rhs) const
	{
		return MakeStringView(*this).Compare(Rhs, ESearchCase::IgnoreCase) > 0;
	}

private:
	[[nodiscard]] friend constexpr inline const CharType* GetData(const TSharedString String)
	{
		return *String;
	}

	[[nodiscard]] friend constexpr inline auto GetNum(const TSharedString String)
	{
		return String.Len();
	}

	[[nodiscard]] friend inline uint32 GetTypeHash(const TSharedString& String)
	{
		return GetTypeHash(MakeStringView(String));
	}

	static inline void AddRef(CharType* Chars);
	static inline void Release(CharType* Chars);

	CharType* Chars = nullptr;

	static constexpr inline CharType NullChar{};

	using FRefCountType = Private::TAtomicRefCount<int32, Private::ERefCountIsQueryable::No>;
	static_assert(sizeof(FRefCountType) == sizeof(int32));
};

template <typename CharType>
inline const TSharedString<CharType> TSharedString<CharType>::Empty;

template <typename CharType>
inline TSharedString<CharType>::TSharedString(TSharedString&& String)
	: Chars(String.Chars)
{
	String.Chars = nullptr;
}

template <typename CharType>
[[nodiscard]] TSharedString<CharType>::TSharedString(const TSharedString& String)
	: Chars(String.Chars)
{
	AddRef(Chars);
}

template <typename CharType>
TSharedString<CharType>& TSharedString<CharType>::operator=(TSharedString&& String)
{
	if (this != &String)
	{
		Release(Chars);
		Chars = String.Chars;
		String.Chars = nullptr;
	}
	return *this;
}

template <typename CharType>
TSharedString<CharType>& TSharedString<CharType>::operator=(const TSharedString& String)
{
	CharType* OldChars = Chars;
	CharType* NewChars = String.Chars;
	AddRef(NewChars);
	Release(OldChars);
	Chars = NewChars;
	return *this;
}

template <typename CharType>
[[nodiscard]] TSharedString<CharType>::TSharedString(const TStringView<CharType> String)
{
	if (const int32 Length = String.Len())
	{
		static_assert(alignof(int32) <= MIN_ALIGNMENT);
		static_assert(alignof(int32) >= alignof(CharType));
		const SIZE_T Size = sizeof(int32) + sizeof(int32) + sizeof(CharType) + sizeof(CharType) * Length;
		int32* const Header = static_cast<int32*>(FMemory::Malloc(Size));
		// RefCount needs to be at the same address as the allocation to make FMemory::Free work as the DeleteFn.
		new((void*)&Header[0]) FRefCountType(1);
		new((void*)&Header[1]) int32(Length);
		Chars = reinterpret_cast<CharType*>(&Header[2]);
		String.CopyString(Chars, Length);
		Chars[Length] = CharType(0);
	}
}

template <typename CharType>
TSharedString<CharType>& TSharedString<CharType>::operator=(const TStringView<CharType> String)
{
	return *this = TSharedString(String);
}

template <typename CharType>
void TSharedString<CharType>::AddRef(CharType* const MaybeNullChars)
{
	if (MaybeNullChars)
	{
		static_assert(sizeof(int32) == sizeof(std::atomic<int32>));
		int32* const Header = reinterpret_cast<int32*>(MaybeNullChars) - 2;
		FRefCountType& RefCount = reinterpret_cast<FRefCountType&>(Header[0]);
		RefCount.AddRef<FMemory::Free>();
	}
}

template <typename CharType>
void TSharedString<CharType>::Release(CharType* const MaybeNullChars)
{
	if (MaybeNullChars)
	{
		static_assert(sizeof(int32) == sizeof(std::atomic<int32>));
		int32* const Header = reinterpret_cast<int32*>(MaybeNullChars) - 2;
		FRefCountType& RefCount = reinterpret_cast<FRefCountType&>(Header[0]);
		RefCount.Release<FMemory::Free>();
	}
}

UE_OPS_NAMESPACE_VISIBLE(FSharedString);
UE_OPS_NAMESPACE_VISIBLE(FAnsiSharedString);
UE_OPS_NAMESPACE_VISIBLE(FWideSharedString);
UE_OPS_NAMESPACE_VISIBLE(FUtf8SharedString);

} // UE
