// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Algo/BinarySearch.h"
#include "Algo/Sort.h"
#include "UObject/NameTypes.h"

/** 
 * A Map of keys to value, implemented as a sorted TArray of TPairs.
 *
 * It has a mostly identical interface to TMap and is designed as a drop in replacement. Keys must be unique,
 * there is no equivalent sorted version of TMultiMap. It uses half as much memory as TMap, but adding and 
 * removing elements is O(n), and finding is O(Log n). In practice it is faster than TMap for low element
 * counts, and slower as n increases, This map is always kept sorted by the key type so cannot be sorted manually.
 */
template <typename InKeyType, typename InValueType, typename ArrayAllocator /*= FDefaultAllocator*/, typename SortPredicate /*= TLess<KeyType>*/ >
class TSortedMap
{
	template <typename OtherKeyType, typename OtherValueType, typename OtherArrayAllocator, typename OtherSortPredicate>
	friend class TSortedMap;

public:
	typedef InKeyType      KeyType;
	typedef InValueType    ValueType;
	UE_DEPRECATED(5.8, "KeyConstPointerType is deprecated - use KeyInitType instead.")
	typedef typename TTypeTraits<KeyType  >::ConstPointerType KeyConstPointerType;
	typedef typename TTypeTraits<KeyType  >::ConstInitType    KeyInitType;
	typedef typename TTypeTraits<ValueType>::ConstInitType    ValueInitType;
	typedef TPair<KeyType, ValueType> ElementType;

	[[nodiscard]] TSortedMap() = default;
	[[nodiscard]] TSortedMap(TSortedMap&&) = default;
	[[nodiscard]] TSortedMap(const TSortedMap&) = default;
	TSortedMap& operator=(TSortedMap&&) = default;
	TSortedMap& operator=(const TSortedMap&) = default;

	/** Constructor for moving elements from a TSortedMap with a different ArrayAllocator. */
	template<typename OtherArrayAllocator>
	[[nodiscard]] TSortedMap(TSortedMap<KeyType, ValueType, OtherArrayAllocator, SortPredicate>&& Other)
		: Pairs(MoveTemp(Other.Pairs))
	{
	}

	/** Constructor for copying elements from a TSortedMap with a different ArrayAllocator. */
	template<typename OtherArrayAllocator>
	[[nodiscard]] TSortedMap(const TSortedMap<KeyType, ValueType, OtherArrayAllocator, SortPredicate>& Other)
		: Pairs(Other.Pairs)
	{
	}

	/** Constructor which gets its elements from a native initializer list */
	[[nodiscard]] TSortedMap(std::initializer_list<TPairInitializer<const KeyType&, const ValueType&>> InitList)
	{
		this->Reserve((int32)InitList.size());
		for (const TPairInitializer<const KeyType&, const ValueType&>& Element : InitList)
		{
			this->Add(Element.Key, Element.Value);
		}
	}

	///////////////////////////////////////////////////
	// Start - intrusive TOptional<TSortedMap> state //
	///////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TSortedMap;

	[[nodiscard]] explicit TSortedMap(FIntrusiveUnsetOptionalState Tag)
		: Pairs(Tag)
	{
	}
	[[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState Tag) const
	{
		return Pairs == Tag;
	}
	/////////////////////////////////////////////////
	// End - intrusive TOptional<TSortedMap> state //
	/////////////////////////////////////////////////

	/** Assignment operator for moving elements from a TSortedMap with a different ArrayAllocator. */
	template<typename OtherArrayAllocator>
	TSortedMap& operator=(TSortedMap<KeyType, ValueType, OtherArrayAllocator, SortPredicate>&& Other)
	{
		Pairs = MoveTemp(Other.Pairs);
		return *this;
	}

	/** Assignment operator for copying elements from a TSortedMap with a different ArrayAllocator. */
	template<typename OtherArrayAllocator>
	TSortedMap& operator=(const TSortedMap<KeyType, ValueType, OtherArrayAllocator, SortPredicate>& Other)
	{
		Pairs = Other.Pairs;
		return *this;
	}

	/** Assignment operator which gets its elements from a native initializer list */
	TSortedMap& operator=(std::initializer_list<TPairInitializer<const KeyType&, const ValueType&>> InitList)
	{
		this->Empty((int32)InitList.size());
		for (const TPairInitializer<const KeyType&, const ValueType&>& Element : InitList)
		{
			this->Add(Element.Key, Element.Value);
		}
		return *this;
	}

	/** Equality operator. This is efficient because pairs are always sorted. */
	UE_FORCEINLINE_HINT bool operator==(const TSortedMap& Other) const
	{
		return Pairs == Other.Pairs;
	}

	/** Inequality operator. This is efficient because pairs are always sorted. */
	UE_FORCEINLINE_HINT bool operator!=(const TSortedMap& Other) const
	{
		return Pairs != Other.Pairs;
	}

	/**
	 * Removes all elements from the map, potentially leaving space allocated for an expected number of elements about to be added.
	 *
	 * @param ExpectedNumElements The number of elements about to be added to the set.
	 */
	UE_FORCEINLINE_HINT void Empty(int32 ExpectedNumElements = 0)
	{
		Pairs.Empty(ExpectedNumElements);
	}

    /** Efficiently empties out the map but preserves all allocations and capacities. */
    UE_FORCEINLINE_HINT void Reset()
    {
        Pairs.Reset();
    }

	/** Shrinks the pair set to avoid slack. */
	UE_FORCEINLINE_HINT void Shrink()
	{
		Pairs.Shrink();
	}

	/** Preallocates enough memory to contain Number elements. */
	UE_FORCEINLINE_HINT void Reserve(int32 Number)
	{
		Pairs.Reserve(Number);
	}

	/**
	 * Returns true if the map is empty and contains no elements. 
	 *
	 * @returns True if the map is empty.
	 * @see Num
	 */
	[[nodiscard]] bool IsEmpty() const
	{
		return Pairs.IsEmpty();
	}

	/** @return The number of elements in the map. */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Num() const
	{
		return Pairs.Num();
	}

	/** @return The number of elements the map can hold before reallocation. */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Max() const
	{
		return Pairs.Max();
	}

	/** 
	 * Helper function to return the amount of memory allocated by this container.
	 * Only returns the size of allocations made directly by the container, not the elements themselves.
	 *
	 * @return number of bytes allocated by this container.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT SIZE_T GetAllocatedSize() const
	{
		return Pairs.GetAllocatedSize();
	}

	/** Tracks the container's memory use through an archive. */
	UE_FORCEINLINE_HINT void CountBytes(FArchive& Ar) const
	{
		Pairs.CountBytes(Ar);
	}

	/**
	 * Sets the value associated with a key.
	 *
	 * @param InKey The key to associate the value with.
	 * @param InValue The value to associate with the key.
	 * @return A reference to the value as stored in the map (only valid until the next change to any key in the map).
	 */
	UE_FORCEINLINE_HINT ValueType& Add(const KeyType&  InKey, const ValueType&  InValue) { return Emplace(         InKey ,          InValue ); }
	UE_FORCEINLINE_HINT ValueType& Add(const KeyType&  InKey,       ValueType&& InValue) { return Emplace(         InKey , MoveTemp(InValue)); }
	UE_FORCEINLINE_HINT ValueType& Add(      KeyType&& InKey, const ValueType&  InValue) { return Emplace(MoveTemp(InKey),          InValue ); }
	UE_FORCEINLINE_HINT ValueType& Add(      KeyType&& InKey,       ValueType&& InValue) { return Emplace(MoveTemp(InKey), MoveTemp(InValue)); }

	/**
	 * Sets a default value associated with a key.
	 *
	 * @param InKey The key to associate the value with.
	 * @return A reference to the value as stored in the map.  The reference is only valid until the next change to any key in the map.
	 */
	UE_FORCEINLINE_HINT ValueType& Add(const KeyType&  InKey) { return Emplace(         InKey ); }
	UE_FORCEINLINE_HINT ValueType& Add(      KeyType&& InKey) { return Emplace(MoveTemp(InKey)); }

	/**
	 * Sets the value associated with a key.
	 *
	 * @param InKey - The key to associate the value with.
	 * @param InValue - The value to associate with the key.
	 * @return A reference to the value as stored in the map (only valid until the next change to any key in the map).
	 */
	template <typename InitKeyType = KeyType, typename InitValueType = ValueType>
	ValueType& Emplace(InitKeyType&& InKey, InitValueType&& InValue)
	{
		ElementType* DataPtr = AllocateMemoryForEmplace(InKey);
	
		::new((void*)DataPtr) ElementType(TPairInitializer<InitKeyType&&, InitValueType&&>(Forward<InitKeyType>(InKey), Forward<InitValueType>(InValue)));

		return DataPtr->Value;
	}

	/**
	 * Sets a default value associated with a key.
	 *
	 * @param InKey The key to associate the value with.
	 * @return A reference to the value as stored in the map (only valid until the next change to any key in the map).
	 */
	template <typename InitKeyType = KeyType>
	ValueType& Emplace(InitKeyType&& InKey)
	{
		ElementType* DataPtr = AllocateMemoryForEmplace(InKey);

		::new((void*)DataPtr) ElementType(TKeyInitializer<InitKeyType&&>(Forward<InitKeyType>(InKey)));

		return DataPtr->Value;
	}

	/**
	 * Removes all value associations for a key.
	 *
	 * @param InKey The key to remove associated values for.
	 * @return The number of values that were associated with the key.
	 */
	inline int32 Remove(KeyInitType InKey, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<ArrayAllocator>())
	{
		int32 RemoveIndex = FindIndex(InKey);

		if (RemoveIndex == INDEX_NONE)
		{
			return 0;
		}

		Pairs.RemoveAt(RemoveIndex, AllowShrinking);

		return 1;
	}

	/**
	 * Returns the key associated with the specified value.  The time taken is O(N) in the number of pairs.
	 *
	 * @param Value The value to search for.
	 * @return A pointer to the key associated with the specified value, or nullptr if the value isn't contained in this map (only valid until the next change to any key in the map).
	 */
	[[nodiscard]] const KeyType* FindKey(ValueInitType Value) const
	{
		for(typename ElementArrayType::TConstIterator PairIt(Pairs);PairIt;++PairIt)
		{
			if(PairIt->Value == Value)
			{
				return &PairIt->Key;
			}
		}
		return nullptr;
	}

	/**
	 * Returns the value associated with a specified key.
	 *
	 * @param Key The key to search for.
	 * @return A pointer to the value associated with the specified key, or nullptr if the key isn't contained in this map.  The pointer (only valid until the next change to any key in the map).
	 */
	[[nodiscard]] inline ValueType* Find(KeyInitType Key)
	{
		int32 FoundIndex = FindIndex(Key);

		if (FoundIndex != INDEX_NONE)
		{
			return &Pairs[FoundIndex].Value;
		}

		return nullptr;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT const ValueType* Find(KeyInitType Key) const
	{
		return const_cast<TSortedMap*>(this)->Find(Key);
	}

	/**
	 * Returns the value associated with a specified key, or if none exists, 
	 * adds a value using the default constructor.
	 *
	 * @param Key The key to search for.
	 * @return A reference to the value associated with the specified key.
	 */
	UE_FORCEINLINE_HINT ValueType& FindOrAdd(const KeyType&  Key) { return FindOrAddImpl(         Key ); }
	UE_FORCEINLINE_HINT ValueType& FindOrAdd(      KeyType&& Key) { return FindOrAddImpl(MoveTemp(Key)); }

	/**
	 * Returns a reference to the value associated with a specified key.
	 *
	 * @param Key The key to search for.
	 * @return The value associated with the specified key, or triggers an assertion if the key does not exist.
	 */
	[[nodiscard]] inline ValueType& FindChecked(KeyInitType Key)
	{
		ValueType* Value = Find(Key);
		check(Value != nullptr);
		return *Value;
	}

	[[nodiscard]] inline const ValueType& FindChecked(KeyInitType Key) const
	{
		const ValueType* Value = Find(Key);
		check(Value != nullptr);
		return *Value;
	}

	/**
	 * Returns the value associated with a specified key.
	 *
	 * @param Key The key to search for.
	 * @return The value associated with the specified key, or the default value for the ValueType if the key isn't contained in this map.
	 */
	[[nodiscard]] inline ValueType FindRef(KeyInitType Key) const
	{
		if (const ValueType* Value = Find(Key))
		{
			return *Value;
		}

		return ValueType();
	}

	/**
	 * Returns the value associated with a specified key.
	 *
	 * @param Key The key to search for.
	 * @param DefaultValue The fallback value if the key is not found.
	 * @return The value associated with the specified key, or DefaultValue if the key isn't contained in this map.
	 */
	[[nodiscard]] inline ValueType FindRef(KeyInitType Key, ValueType DefaultValue) const
	{
		if (const ValueType* Value = Find(Key))
		{
			return *Value;
		}

		return DefaultValue;
	}

	/**
	 * Finds any pair in the map and returns a pointer to it.
	 * Callers should not depend on particular patterns in the behaviour of this function.
	 * @return A pointer to an arbitrary pair, or nullptr if the container is empty.
	 */
	[[nodiscard]] ElementType* FindArbitraryElement()
	{
		// The goal of this function is to be fast, and so the implementation may be improved at any time even if it gives different results.

		return Pairs.IsEmpty() ? nullptr : &Pairs.Last();
	}
	[[nodiscard]] const ElementType* FindArbitraryElement() const
	{
		return const_cast<TSortedMap*>(this)->FindArbitraryElement();
	}

	/**
	 * Checks if map contains the specified key.
	 *
	 * @param Key The key to check for.
	 * @return true if the map contains the key.
	 */
	[[nodiscard]] inline bool Contains(KeyInitType Key) const
	{
		if (Find(Key))
		{
			return true;
		}
		return false;
	}

	/**
	 * Returns the unique keys contained within this map.
	 *
	 * @param OutKeys Upon return, contains the set of unique keys in this map.
	 * @return The number of unique keys in the map.
	 */
	template<typename Allocator> int32 GetKeys(TArray<KeyType, Allocator>& OutKeys) const
	{
		for (typename ElementArrayType::TConstIterator PairIt(Pairs); PairIt; ++PairIt)
		{
			OutKeys.Add(PairIt->Key);
		}

		return OutKeys.Num();
	}

	/**
	 * Generates an array from the keys in this map.
	 */
	template<typename Allocator> void GenerateKeyArray(TArray<KeyType, Allocator>& OutArray) const
	{
		OutArray.Empty(Pairs.Num());
		for(typename ElementArrayType::TConstIterator PairIt(Pairs);PairIt;++PairIt)
		{
			OutArray.Add(PairIt->Key);
		}
	}

	/**
	 * Generates an array from the values in this map.
	 */
	template<typename Allocator> void GenerateValueArray(TArray<ValueType, Allocator>& OutArray) const
	{
		OutArray.Empty(Pairs.Num());
		for(typename ElementArrayType::TConstIterator PairIt(Pairs);PairIt;++PairIt)
		{
			OutArray.Add(PairIt->Value);
		}
	}

	/**
	 * Describes the map's contents through an output device.
	 *
	 * @param Ar The output device to describe the map's contents through.
	 */
	void Dump(FOutputDevice& Ar)
	{
		Pairs.Dump(Ar);
	}

	/**
	 * Removes the pair with the specified key and copies the value that was removed to the ref parameter.
	 *
	 * @param Key The key to search for
	 * @param OutRemovedValue If found, the value that was removed (not modified if the key was not found)
	 * @return Whether or not the key was found
	 */
	inline bool RemoveAndCopyValue(KeyInitType Key, ValueType& OutRemovedValue, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<ArrayAllocator>())
	{
		int32 FoundIndex = FindIndex(Key);
		if (FoundIndex == INDEX_NONE)
		{
			return false;
		}

		OutRemovedValue = MoveTemp(Pairs[FoundIndex].Value);
		Pairs.RemoveAt(FoundIndex, AllowShrinking);
		return true;
	}
	
	/**
	 * Finds a pair with the specified key, removes it from the map, and returns the value part of the pair.
	 * If no pair was found, an exception is thrown.
	 *
	 * @param Key The key to search for
	 * @return Whether or not the key was found
	 */
	inline ValueType FindAndRemoveChecked(KeyInitType Key, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<ArrayAllocator>())
	{
		int32 FoundIndex = FindIndex(Key);
		check(FoundIndex != INDEX_NONE);
		
		ValueType OutRemovedValue = MoveTemp(Pairs[FoundIndex].Value);
		Pairs.RemoveAt(FoundIndex, AllowShrinking);
		return OutRemovedValue;
	}

	/**
	 * Move all items from another map into our map (if any keys are in both, the value from the other map wins) and empty the other map.
	 *
	 * @param OtherMap The other map of items to move the elements from.
	 */
	template<typename OtherArrayAllocator, typename OtherSortPredicate>
	void Append(TSortedMap<KeyType, ValueType, OtherArrayAllocator, OtherSortPredicate>&& OtherMap)
	{
		this->Reserve(this->Num() + OtherMap.Num());
		for (auto& Pair : OtherMap)
		{
			this->Add(MoveTemp(Pair.Key), MoveTemp(Pair.Value));
		}

		OtherMap.Reset();
	}

	/**
	 * Add all items from another map to our map (if any keys are in both, the value from the other map wins).
	 *
	 * @param OtherMap The other map of items to add.
	 */
	template<typename OtherArrayAllocator, typename OtherSortPredicate>
	void Append(const TSortedMap<KeyType, ValueType, OtherArrayAllocator, OtherSortPredicate>& OtherMap)
	{
		this->Reserve(this->Num() + OtherMap.Num());
		for (auto& Pair : OtherMap)
		{
			this->Add(Pair.Key, Pair.Value);
		}
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ValueType& operator[](KeyInitType Key)
	{
		return this->FindChecked(Key);
	}
	[[nodiscard]] UE_FORCEINLINE_HINT const ValueType& operator[](KeyInitType Key) const
	{
		return this->FindChecked(Key);
	}

	// Interface functions to match TMap/TSet

	/** @return The non-inclusive maximum index of elements in the map (was: the max valid index of the elements in the sparse storage). */
	UE_DEPRECATED(5.7 until 5.9, "GetMaxIndex() should be replaced with Num() - 1.")
	[[nodiscard]] UE_FORCEINLINE_HINT int32 GetMaxIndex() const
	{
		// We don't want to deprecate the function entirely, because it should mirror the the existing TMap API, but it's been
		// returning the wrong value and we want to fix that.
		//
		// The deprecation macro lists "5.7 until 5.9", and from 5.9 this function should be de-deprecated and the return value should become "return Pairs.Num();".
		return Pairs.Num() - 1;
	}

	/**
	 * Checks whether an element id is valid.
	 * @param Id - The element id to check.
	 * @return true if the element identifier refers to a valid element in this map.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValidId(FSetElementId Id) const
	{
		return Pairs.IsValidIndex(Id.AsInteger());
	}

	/** Return a mapped pair by internal identifier. Element must be valid (see @IsValidId). */
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& Get(FSetElementId Id)
	{
		return Pairs[Id.AsInteger()];
	}

	/** Return a mapped pair by internal identifier.  Element must be valid (see @IsValidId).*/
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType& Get(FSetElementId Id) const
	{
		return Pairs[Id.AsInteger()];
	}

private:
	typedef TArray<ElementType, ArrayAllocator> ElementArrayType;

	/** Implementation of find and add */
	template <typename ArgType>
	[[nodiscard]] inline ValueType& FindOrAddImpl(ArgType&& Key)
	{
		if (ValueType* Value = Find(Key))
		{
			return *Value;
		}

		return Add(Forward<ArgType>(Key));
	}

	/** Find index of key */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 FindIndex(KeyInitType Key) const
	{
		return Algo::BinarySearchBy(Pairs, Key, FKeyForward(), SortPredicate());
	}

	/** Allocates raw memory for emplacing */
	template <typename InitKeyType>
	inline ElementType* AllocateMemoryForEmplace(InitKeyType&& InKey)
	{
		int32 InsertIndex = Algo::LowerBoundBy(Pairs, InKey, FKeyForward(), SortPredicate());
		check(InsertIndex >= 0 && InsertIndex <= Pairs.Num());

		ElementType* DataPtr = nullptr;
		// Since we returned lower bound we already know InKey <= InsertIndex key. So if InKey is not < InsertIndex key, they must be equal
		if (Pairs.IsValidIndex(InsertIndex) && !SortPredicate()(InKey, Pairs[InsertIndex].Key))
		{
			// Replacing element, delete old one
			DataPtr = Pairs.GetData() + InsertIndex;
			DestructItems(DataPtr, 1);
		}
		else
		{
			// Adding new one, this may reallocate Pairs
			Pairs.InsertUninitialized(InsertIndex, 1);
			DataPtr = Pairs.GetData() + InsertIndex;
		}

		return DataPtr;
	}

	/** Forwards sorting into Key of pair */
	struct FKeyForward
	{
		UE_FORCEINLINE_HINT const KeyType& operator()(const ElementType& Pair) const
		{
			return Pair.Key;
		}
	};

	/** The base of TSortedMap iterators */
	template<bool bConst>
	class TBaseIterator
	{
	public:
		typedef std::conditional_t<bConst,typename ElementArrayType::TConstIterator,typename ElementArrayType::TIterator> PairItType;
	private:
		typedef std::conditional_t<bConst,const TSortedMap,TSortedMap> MapType;
		typedef std::conditional_t<bConst,const KeyType,KeyType> ItKeyType;
		typedef std::conditional_t<bConst,const ValueType,ValueType> ItValueType;
		typedef std::conditional_t<bConst,const typename ElementArrayType::ElementType, typename ElementArrayType::ElementType> PairType;

	protected:
		[[nodiscard]] UE_FORCEINLINE_HINT TBaseIterator(const PairItType& InElementIt)
			: PairIt(InElementIt)
		{
		}

	public:
		UE_FORCEINLINE_HINT TBaseIterator& operator++()
		{
			++PairIt;
			return *this;
		}

		/** conversion to "bool" returning true if the iterator is valid. */
		[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
		{
			return !!PairIt; 
		}

		[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TBaseIterator& Rhs) const
		{
			return PairIt == Rhs.PairIt;
		}
		[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TBaseIterator& Rhs) const
		{
			return PairIt != Rhs.PairIt;
		}

		UE_FORCEINLINE_HINT ItKeyType& Key() const
		{
			return PairIt->Key;
		}
		UE_FORCEINLINE_HINT ItValueType& Value() const
		{
			return PairIt->Value;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT FSetElementId GetId() const
		{
			return FSetElementId::FromInteger(PairIt.GetIndex());
		}

		UE_FORCEINLINE_HINT PairType& operator*() const
		{
			return  *PairIt;
		}
		UE_FORCEINLINE_HINT PairType* operator->() const
		{
			return &*PairIt;
		}

	protected:
		PairItType PairIt;
	};

	/** The base of TSortedMap reverse iterators. */
	template<bool bConst>
	class TBaseReverseIterator
	{
		// Once we add reverse iterator to TArray, this class and TBaseIterator could be merged with a template parameter for forward vs reverse.
	private:
		typedef std::conditional_t<bConst, const TSortedMap, TSortedMap> MapType;
		typedef std::conditional_t<bConst, const KeyType, KeyType> ItKeyType;
		typedef std::conditional_t<bConst, const ValueType, ValueType> ItValueType;
		typedef typename ElementArrayType::SizeType SizeType;

	public:
		typedef std::conditional_t<bConst, const typename ElementArrayType::ElementType, typename ElementArrayType::ElementType> PairType;

	protected:
		[[nodiscard]] UE_FORCEINLINE_HINT TBaseReverseIterator(PairType* InData, SizeType InNum)
			: Data(InData), Index(InNum-1)
		{
		}

	public:
		inline TBaseReverseIterator& operator++()
		{
			checkf(Index != static_cast<SizeType>(-1), TEXT("Incrementing an invalid iterator is illegal"));
			--Index;
			return *this;
		}

		/** conversion to "bool" returning true if the iterator is valid. */
		[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
		{
			return Index != static_cast<SizeType>(-1);
		}

		[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TBaseReverseIterator& Rhs) const
		{
			return Index == Rhs.Index;
		}
		[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TBaseReverseIterator& Rhs) const
		{
			return Index != Rhs.Index;
		}

		UE_FORCEINLINE_HINT ItKeyType& Key() const
		{
			return Data[Index].Key;
		}
		UE_FORCEINLINE_HINT ItValueType& Value() const
		{
			return Data[Index].Value;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT FSetElementId GetId() const
		{
			return FSetElementId::FromInteger(Index);
		}

		UE_FORCEINLINE_HINT PairType& operator*() const
		{
			return  Data[Index];
		}
		UE_FORCEINLINE_HINT PairType* operator->() const
		{
			return &Data[Index];
		}

	protected:
		PairType* Data;
		SizeType Index;
	};

	/** An array of the key-value pairs in the map */
	ElementArrayType Pairs;

public:

	/** Map iterator */
	class TIterator : public TBaseIterator<false>
	{
	public:

		[[nodiscard]] UE_FORCEINLINE_HINT TIterator(TSortedMap& InMap)
			: TBaseIterator<false>(InMap.Pairs.CreateIterator())
		{
		}

		[[nodiscard]] UE_FORCEINLINE_HINT TIterator(const typename TBaseIterator<false>::PairItType& InPairIt)
			: TBaseIterator<false>(InPairIt)
		{
		}

		/** Removes the current pair from the map */
		UE_FORCEINLINE_HINT void RemoveCurrent()
		{
			TBaseIterator<false>::PairIt.RemoveCurrent();
		}
	};

	/** Const map iterator */
	class TConstIterator : public TBaseIterator<true>
	{
	public:
		[[nodiscard]] UE_FORCEINLINE_HINT TConstIterator(const TSortedMap& InMap)
			: TBaseIterator<true>(InMap.Pairs.CreateConstIterator())
		{
		}

		[[nodiscard]] UE_FORCEINLINE_HINT TConstIterator(const typename TBaseIterator<true>::PairItType& InPairIt)
			: TBaseIterator<true>(InPairIt)
		{
		}
	};

	/** Reverse Map iterator */
	class TReverseIterator : public TBaseReverseIterator<false>
	{
	public:
		[[nodiscard]] UE_FORCEINLINE_HINT TReverseIterator(TSortedMap& InMap)
			: TBaseReverseIterator<false>(InMap.Pairs.GetData(), InMap.Pairs.Num())
		{
		}

		// Add constructor from PairItType and RemoveCurrent once we have reverse iterators on TArray
	};

	/** Const map iterator */
	class TConstReverseIterator : public TBaseReverseIterator<true>
	{
	public:
		[[nodiscard]] UE_FORCEINLINE_HINT TConstReverseIterator(const TSortedMap& InMap)
			: TBaseReverseIterator<true>(InMap.Pairs.GetData(), InMap.Pairs.Num())
		{
		}

		// Add constructor from PairItType once we have reverse iterators on TArray
	};

	/** Iterates over values associated with a specified key in a const map. This will be at most one value because keys must be unique */
	class TConstKeyIterator : public TBaseIterator<true>
	{
		using Super = TBaseIterator<true>;

	public:
		[[nodiscard]] inline TConstKeyIterator(const TSortedMap& InMap, KeyInitType InKey)
			: Super(InMap.Pairs.CreateConstIterator())
		{
			int32 NewIndex = InMap.FindIndex(InKey);
		
			if (NewIndex != INDEX_NONE)
			{
				Super::PairIt += NewIndex;
			}
			else
			{
				Super::PairIt.SetToEnd();
			}
		}

		inline TConstKeyIterator& operator++()
		{
			Super::PairIt.SetToEnd();
			return *this;
		}
	};

	/** Iterates over values associated with a specified key in a map. This will be at most one value because keys must be unique */
	class TKeyIterator : public TBaseIterator<false>
	{
		using Super = TBaseIterator<false>;

	public:
		[[nodiscard]] inline TKeyIterator(TSortedMap& InMap, KeyInitType InKey)
			: Super(InMap.Pairs.CreateIterator())
		{
			int32 NewIndex = InMap.FindIndex(InKey);

			if (NewIndex != INDEX_NONE)
			{
				Super::PairIt += NewIndex;
			}
			else
			{
				Super::PairIt.SetToEnd();
			}
		}

		inline TKeyIterator& operator++()
		{
			Super::PairIt.SetToEnd();
			return *this;
		}

		/** Removes the current key-value pair from the map. */
		inline void RemoveCurrent()
		{
			Super::PairIt.RemoveCurrent();
			Super::PairIt.SetToEnd();
		}
	};

	/** Creates an iterator over all the pairs in this map */
	[[nodiscard]] UE_FORCEINLINE_HINT TIterator CreateIterator()
	{
		return TIterator(*this);
	}

	/** Creates a const iterator over all the pairs in this map */
	[[nodiscard]] UE_FORCEINLINE_HINT TConstIterator CreateConstIterator() const
	{
		return TConstIterator(*this);
	}

	/** Creates an iterator over the values associated with a specified key in a map. This will be at most one value because keys must be unique */
	[[nodiscard]] UE_FORCEINLINE_HINT TKeyIterator CreateKeyIterator(KeyInitType InKey)
	{
		return TKeyIterator(*this, InKey);
	}

	/** Creates a const iterator over the values associated with a specified key in a map. This will be at most one value because keys must be unique */
	[[nodiscard]] UE_FORCEINLINE_HINT TConstKeyIterator CreateConstKeyIterator(KeyInitType InKey) const
	{
		return TConstKeyIterator(*this, InKey);
	}

	/** Ranged For iterators. Unlike normal TMap these are not the same as the normal iterator for performance reasons */
	typedef typename ElementArrayType::RangedForIteratorType RangedForIteratorType;
	typedef typename ElementArrayType::RangedForConstIteratorType RangedForConstIteratorType;

public:
	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT RangedForIteratorType		begin()	      { return Pairs.begin(); }
	[[nodiscard]] UE_FORCEINLINE_HINT RangedForConstIteratorType	begin() const { return Pairs.begin(); }
	[[nodiscard]] UE_FORCEINLINE_HINT RangedForIteratorType		end()         { return Pairs.end(); }
	[[nodiscard]] UE_FORCEINLINE_HINT RangedForConstIteratorType	end() const   { return Pairs.end(); }

	friend struct TSortedMapPrivateFriend;
};

DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT((template <typename KeyType, typename ValueType, typename ArrayAllocator, typename SortPredicate>), (TSortedMap<KeyType, ValueType, ArrayAllocator, SortPredicate>));

struct TSortedMapPrivateFriend
{
	template <typename KeyType, typename ValueType, typename ArrayAllocator, typename SortPredicate>
	static void Serialize(FArchive& Ar, TSortedMap<KeyType, ValueType, ArrayAllocator, SortPredicate>& Map)
	{
		Ar << Map.Pairs;

		if (Ar.IsLoading())
		{
			// We need to resort, in case the sorting is not consistent with what it was before
			Algo::SortBy(Map.Pairs, typename TSortedMap<KeyType, ValueType, ArrayAllocator, SortPredicate>::FKeyForward(), SortPredicate());
		}
	}
};

/** Serializer. */
template <typename KeyType, typename ValueType, typename ArrayAllocator, typename SortPredicate>
inline FArchive& operator<<(FArchive& Ar, TSortedMap<KeyType, ValueType, ArrayAllocator, SortPredicate>& Map)
{
	TSortedMapPrivateFriend::Serialize(Ar, Map);
	return Ar;
}

