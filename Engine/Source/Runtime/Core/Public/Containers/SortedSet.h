// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/BinarySearch.h"
#include "Algo/IndexOf.h"
#include "Algo/Sort.h"
#include "Containers/ArrayView.h"
#include "Containers/SetUtilities.h"
#include "UObject/NameTypes.h"

template <typename ElementType, typename ArrayAllocator, typename SortPredicate>
FArchive& operator<<(FArchive& Ar, TSortedSet<ElementType, ArrayAllocator, SortPredicate>& Set);

/** 
 * A set of values, implemented as a sorted TArray of elements.
 *
 * It has a mostly identical interface to TSet and is designed as a drop in replacement. Keys must be unique.
 * It uses half as much memory as TSet and has a smaller static footprint, but adding and removing elements is O(n),
 * and finding is O(Log n). In practice it is faster than TSet for low element counts, and slower as n increases.
 * This set is always kept sorted by key so cannot be sorted manually.
 */
template <typename InElementType, typename ArrayAllocator /*= FDefaultAllocator*/, typename SortPredicate /*= TLess<InElementType>*/>
class TSortedSet
{
	template <typename OtherElementType, typename OtherArrayAllocator, typename OtherSortPredicate>
	friend class TSortedSet;

	friend FArchive& operator<< <InElementType, ArrayAllocator, SortPredicate>(FArchive& Ar, TSortedSet& Set);

public:
	using ElementType = InElementType;
	using KeyInitType = typename TTypeTraits<ElementType>::ConstPointerType;

	[[nodiscard]] TSortedSet() = default;
	[[nodiscard]] TSortedSet(TSortedSet&&) = default;
	[[nodiscard]] TSortedSet(const TSortedSet&) = default;
	TSortedSet& operator=(TSortedSet&&) = default;
	TSortedSet& operator=(const TSortedSet&) = default;

	/** Constructor for moving elements from a TSortedSet with a different ArrayAllocator. */
	template<typename OtherArrayAllocator>
	[[nodiscard]] TSortedSet(TSortedSet<InElementType, OtherArrayAllocator, SortPredicate>&& Other)
		: Elements(MoveTemp(Other.Elements))
	{
	}

	/** Constructor for copying elements from a TSortedSet with a different ArrayAllocator. */
	template<typename OtherArrayAllocator>
	[[nodiscard]] TSortedSet(const TSortedSet<InElementType, OtherArrayAllocator, SortPredicate>& Other)
		: Elements(Other.Elements)
	{
	}

	/** Constructor which gets its elements from a native initializer list */
	[[nodiscard]] TSortedSet(std::initializer_list<ElementType> InitList)
	{
		this->Reserve((int32)InitList.size());
		this->Append(InitList);
	}

	///////////////////////////////////////////////////
	// Start - intrusive TOptional<TSortedSet> state //
	///////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TSortedSet;

	[[nodiscard]] explicit TSortedSet(FIntrusiveUnsetOptionalState Tag)
		: Elements(Tag)
	{
	}
	[[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState Tag) const
	{
		return Elements == Tag;
	}
	/////////////////////////////////////////////////
	// End - intrusive TOptional<TSortedSet> state //
	/////////////////////////////////////////////////

	/** Assignment operator for moving elements from a TSortedSet with a different ArrayAllocator. */
	template<typename OtherArrayAllocator>
	TSortedSet& operator=(TSortedSet<ElementType, OtherArrayAllocator, SortPredicate>&& Other)
	{
		Elements = MoveTemp(Other.Elements);
		return *this;
	}

	/** Assignment operator for copying elements from a TSortedSet with a different ArrayAllocator. */
	template<typename OtherArrayAllocator>
	TSortedSet& operator=(const TSortedSet<ElementType, OtherArrayAllocator, SortPredicate>& Other)
	{
		Elements = Other.Elements;
		return *this;
	}

	/** Assignment operator which gets its elements from a native initializer list */
	TSortedSet& operator=(std::initializer_list<ElementType> InitList)
	{
		this->Empty((int32)InitList.size());
		this->Append(InitList);
		return *this;
	}

	/** Equality operator. This is efficient because elements are always sorted. */
	UE_FORCEINLINE_HINT bool operator==(const TSortedSet& Other) const
	{
		return Elements == Other.Elements;
	}

	/** Inequality operator. This is efficient because elements are always sorted. */
	UE_FORCEINLINE_HINT bool operator!=(const TSortedSet& Other) const
	{
		return Elements != Other.Elements;
	}

	/**
	 * Removes all elements from the set, potentially leaving space allocated for an expected number of elements about to be added.
	 *
	 * @param ExpectedNumElements The number of elements about to be added to the set.
	 */
	UE_FORCEINLINE_HINT void Empty(int32 ExpectedNumElements = 0)
	{
		Elements.Empty(ExpectedNumElements);
	}

	/** Efficiently empties out the set but preserves all allocations and capacities. */
	UE_FORCEINLINE_HINT void Reset()
	{
		Elements.Reset();
	}

	/** Shrinks the set to avoid slack. */
	UE_FORCEINLINE_HINT void Shrink()
	{
		Elements.Shrink();
	}

	/** Preallocates enough memory to contain Number elements. */
	UE_FORCEINLINE_HINT void Reserve(int32 Number)
	{
		Elements.Reserve(Number);
	}

	/**
	 * Returns true if the set is empty and contains no elements. 
	 *
	 * @returns True if the set is empty.
	 * @see Num
	 */
	[[nodiscard]] bool IsEmpty() const
	{
		return Elements.IsEmpty();
	}

	/** @return The number of elements in the set. */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Num() const
	{
		return Elements.Num();
	}

	/** @return The number of elements the set can hold before reallocation. */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Max() const
	{
		return Elements.Max();
	}

	/** @return The non-inclusive maximum index of elements in the set. */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 GetMaxIndex() const
	{
		return Elements.Num();
	}

	/** 
	 * Helper function to return the amount of memory allocated by this container.
	 * Only returns the size of allocations made directly by the container, not the elements themselves.
	 *
	 * @return number of bytes allocated by this container.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT SIZE_T GetAllocatedSize() const
	{
		return Elements.GetAllocatedSize();
	}

	/** Tracks the container's memory use through an archive. */
	UE_FORCEINLINE_HINT void CountBytes(FArchive& Ar) const
	{
		Elements.CountBytes(Ar);
	}

	/**
	 * Adds an element to the set.
	 *
	 * @param	InValue	The value to add.
	 * @param	bIsAlreadyInSetPtr	[out]	Optional pointer to bool that will be set depending on whether element is already in set
	 * @return A handle to the value in the set (only valid until the next change to the set).
	 */
	UE_FORCEINLINE_HINT FSetElementId Add(const ElementType& InValue, bool* bIsAlreadyInSetPtr = nullptr)
	{
		return Emplace(InValue, bIsAlreadyInSetPtr);
	}
	UE_FORCEINLINE_HINT FSetElementId Add(ElementType&& InValue, bool* bIsAlreadyInSetPtr = nullptr)
	{
		return Emplace(MoveTemp(InValue), bIsAlreadyInSetPtr);
	}

	/**
	 * Adds an element to the set.
	 *
	 * @param	InInitArg	The argument to be forwarded to the set element's constructor.
	 * @param	bIsAlreadyInSetPtr	[out]	Optional pointer to bool that will be set depending on whether element is already in set
	 * @return A handle to the value in the set (only valid until the next change to the set).
	 */
	template <typename InitArgType = ElementType>
	FSetElementId Emplace(InitArgType&& InInitArg, bool* bIsAlreadyInSetPtr = nullptr)
	{
		ElementType* DataPtr = AllocateMemoryForEmplace(InInitArg, bIsAlreadyInSetPtr);
	
		::new((void*)DataPtr) ElementType(Forward<InitArgType>(InInitArg));

		return FSetElementId::FromInteger((int32)(DataPtr - Elements.GetData()));
	}

	/**
	 * Removes all elements from the set matching the specified key.
	 *
	 * @param	InKey	The key to match elements against.
	 * @return	The number of elements removed.
	 */
	UE_FORCEINLINE_HINT int32 Remove(KeyInitType InKey, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<ArrayAllocator>())
	{
		int32 RemoveIndex = FindIndex(InKey);
		if (RemoveIndex == INDEX_NONE)
		{
			return 0;
		}

		Elements.RemoveAt(RemoveIndex, AllowShrinking);
		return 1;
	}

	/**
	 * Returns the value associated with a specified key.
	 *
	 * @param Key The key to search for.
	 * @return A pointer to the value associated with the specified key, or nullptr if the key isn't contained in this set.  The pointer (only valid until the next change to any key in the set).
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType* Find(KeyInitType Key)
	{
		int32 FoundIndex = FindIndex(Key);
		if (FoundIndex != INDEX_NONE)
		{
			return &Elements[FoundIndex];
		}

		return nullptr;
	}
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType* Find(KeyInitType Key) const
	{
		return const_cast<TSortedSet*>(this)->Find(Key);
	}

	/**
	 * Adds an element to the set if not already present and returns a reference to the added or existing element.
	 *
	 * @param	InValue	The value to add.
	 * @param	bIsAlreadyInSetPtr	[out]	Optional pointer to bool that will be set depending on whether element is already in set
	 * @return A handle to the value in the set (only valid until the next change to the set).
	 */
	UE_FORCEINLINE_HINT ElementType& FindOrAdd(const ElementType& InValue, bool* bIsAlreadyInSetPtr = nullptr)
	{
		return FindOrAddImpl(InValue, bIsAlreadyInSetPtr);
	}
	UE_FORCEINLINE_HINT ElementType& FindOrAdd(ElementType&& InValue, bool* bIsAlreadyInSetPtr = nullptr)
	{
		return FindOrAddImpl(MoveTemp(InValue), bIsAlreadyInSetPtr);
	}

	/**
	 * Finds any element in the set and returns a pointer to it.
	 * Callers should not depend on particular patterns in the behavior of this function.
	 * @return A pointer to an arbitrary element, or nullptr if the container is empty.
	 */
	[[nodiscard]] ElementType* FindArbitraryElement()
	{
		// The goal of this function is to be fast, and so the implementation may be improved at any time even if it gives different results.

		return Elements.IsEmpty() ? nullptr : &Elements.Last();
	}
	[[nodiscard]] const ElementType* FindArbitraryElement() const
	{
		return const_cast<TSortedSet*>(this)->FindArbitraryElement();
	}

	/**
	 * Checks if set contains the specified key.
	 *
	 * @param Key The key to check for.
	 * @return true if the set contains the key.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool Contains(KeyInitType Key) const
	{
		return FindIndex(Key) != INDEX_NONE;
	}

	/**
	 * Returns a TArray of the elements.
	 */
	TArray<ElementType> Array() const&
	{
		return TArray<ElementType>(Elements);
	}
	TArray<ElementType> Array() &&
	{
		return TArray<ElementType>(MoveTemp(Elements));
	}

	/**
	 * Describes the set's contents through an output device.
	 *
	 * @param Ar The output device to describe the set's contents through.
	 */
	void Dump(FOutputDevice& Ar)
	{
		Ar.Logf(TEXT("TSortedSet: %i elements"), Elements.Num());
	}

private:
	template <bool bMove>
	void AppendImpl(std::conditional_t<bMove, ElementType, const ElementType>* Ptr, int32 Count)
	{
		// Insert new unsorted elements at the end of sorted elements, then sort them all together into the right place at the end.
		//
		// Needs to handle duplicates in both the existing set and within new elements.

		if (Count == 0)
		{
			return;
		}

		// Reserve space for the new elements - we may not need this many if there are duplicates
		Elements.Reserve(Elements.Num() + Count);

		// Get a view over the sorted elements - even if Elements grows in the loop below, this won't be invalidated due to the Reserve above
		TArrayView<ElementType> SortedView = Elements;

		for (;;)
		{
			// Find existing element in sorted elements
			int32 InsertIndex = 0;
			if (!SortedView.IsEmpty())
			{
				// If the new element is strictly 'greater' than the last of the sorted elements, then insert at the end, otherwise binary search for the insertion point
				if (SortPredicate()(SortedView.Last(), *Ptr))
				{
					InsertIndex = SortedView.Num();
				}
				else
				{
					InsertIndex = Algo::LowerBound(SortedView, *Ptr, SortPredicate());
				}
			}

			// If the element at the insertion point is equivalent to ('not less than') the new element, then we can insert it in-place without breaking the sort order
			if (InsertIndex < SortedView.Num() && !SortPredicate()(*Ptr, Elements[InsertIndex]))
			{
				// Replace duplicate element by constructing it over the previous one - this will still leave the sorted elements in order
				ElementType* OldElement = Elements.GetData() + InsertIndex;
				DestructItem(OldElement);
				if constexpr (bMove)
				{
					::new ((void*)OldElement) ElementType(MoveTemp(*Ptr));
				}
				else
				{
					::new ((void*)OldElement) ElementType(*Ptr);
				}
			}
			else
			{
				// Find existing element in unsorted elements
				ElementType* ExistingElement = nullptr;
				int32 NumUnsortedElements = Elements.Num() - SortedView.Num();
				if (NumUnsortedElements > 0)
				{
					ExistingElement = MakeArrayView(Elements.GetData() + SortedView.Num(), NumUnsortedElements).FindByPredicate([Ptr](const ElementType& Val) { return !SortPredicate()(*Ptr, Val) && !SortPredicate()(Val, *Ptr); });
				}

				if (ExistingElement)
				{
					// Replace duplicate element by constructing it over the previous one.
					DestructItem(ExistingElement);
					if constexpr (bMove)
					{
						::new ((void*)ExistingElement) ElementType(MoveTemp(*Ptr));
					}
					else
					{
						::new ((void*)ExistingElement) ElementType(*Ptr);
					}
				}
				else
				{
					// Add the element to the end of the container
					if constexpr (bMove)
					{
						Elements.Add(MoveTemp(*Ptr));
					}
					else
					{
						Elements.Add(*Ptr);
					}

					// If the new element is precisely 'greater than' all existing sorted elements and we haven't added any unsorted elements yet, then just expand
					// the sorted view to include the new element.
					if (InsertIndex == SortedView.Num() && Elements.Num() == SortedView.Num() + 1)
					{
						SortedView = Elements;
					}
				}
			}

			--Count;
			if (Count == 0)
			{
				// If we still have unsorted elements, sort them into the rest of the container
				if (SortedView.Num() != Elements.Num())
				{
					Algo::Sort(Elements, SortPredicate());
				}
				return;
			}

			++Ptr;
		}
	}

public:
	/**
	 * Adds all items from another set into our set (if any elements are in both, the value from the other set wins).
	 * If the source is an rvalue container, the elements will be moved and the container will be left empty.
	 *
	 * @param Other The range of elements to add.
	 */
	template <typename OtherArrayAllocator, typename OtherSortPredicate>
	void Append(TSortedSet<ElementType, OtherArrayAllocator, OtherSortPredicate>&& Other)
	{
		this->AppendImpl<true>(Other.Elements.GetData(), Other.Elements.Num());
		Other.Reset();
	}
	template <typename OtherArrayAllocator, typename OtherSortPredicate>
	void Append(const TSortedSet<ElementType, OtherArrayAllocator, OtherSortPredicate>& Other)
	{
		this->AppendImpl<false>(Other.Elements.GetData(), Other.Elements.Num());
	}
	template <typename OtherArrayAllocator>
	void Append(TArray<ElementType, OtherArrayAllocator>&& Other)
	{
		this->AppendImpl<true>(Other.GetData(), Other.Num());
		Other.Reset();
	}
	void Append(TArrayView<const ElementType> Other)
	{
		this->AppendImpl<false>(Other.GetData(), Other.Num());
	}
	template <typename OtherArrayAllocator>
	void Append(std::initializer_list<ElementType> Other)
	{
		this->AppendImpl<false>(Other.GetData(), (int32)Other.Num());
	}

	/**
	 * Checks whether an element id is valid.
	 * @param Id - The element id to check.
	 * @return true if the element identifier refers to a valid element in this set.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValidId(FSetElementId Id) const
	{
		return Elements.IsValidIndex(Id.AsInteger());
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& operator[](FSetElementId Id)
	{
		return Elements[Id.AsInteger()];
	}
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType& operator[](FSetElementId Id) const
	{
		return (*const_cast<TSortedSet*>(this))[Id];
	}
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& Get(FSetElementId Id)
	{
		return (*this)[Id];
	}
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType& Get(FSetElementId Id) const
	{
		return (*this)[Id];
	}

private:
	using ElementArrayType = TArray<ElementType, ArrayAllocator>;

	/** Implementation of find and add */
	template <typename InitArgType>
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& FindOrAddImpl(InitArgType&& InInitArg, bool* bIsAlreadyInSetPtr)
	{
		int32 InsertIndex = Algo::LowerBound(Elements, InInitArg, SortPredicate());

		// Since we returned lower bound we already know InInitArg <= InsertIndex key. So if InInitArg is not < InsertIndex key, they must be equal
		bool bAlreadyExists = InsertIndex < Elements.Num() && !SortPredicate()(InInitArg, Elements[InsertIndex]);

		ElementType* DataPtr = nullptr;
		if (!bAlreadyExists)
		{
			Elements.EmplaceAt(InsertIndex, Forward<InitArgType>(InInitArg));
		}

		if (bIsAlreadyInSetPtr)
		{
			*bIsAlreadyInSetPtr = bAlreadyExists;
		}

		return *(Elements.GetData() + InsertIndex);
	}

	/** Find index of key */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 FindIndex(KeyInitType Key) const
	{
		return Algo::BinarySearch(Elements, Key, SortPredicate());
	}

	/** Allocates raw memory for emplacing */
	template <typename InitArgType>
	UE_FORCEINLINE_HINT ElementType* AllocateMemoryForEmplace(InitArgType&& InInitArg, bool* bIsAlreadyInSetPtr = nullptr)
	{
		int32 InsertIndex = Algo::LowerBound(Elements, InInitArg, SortPredicate());

		// Since we returned lower bound we already know InInitArg <= InsertIndex key. So if InInitArg is not < InsertIndex key, they must be equal
		bool bAlreadyExists = InsertIndex < Elements.Num() && !SortPredicate()(InInitArg, Elements[InsertIndex]);

		ElementType* DataPtr = nullptr;
		if (bAlreadyExists)
		{
			// Replacing element, delete old one
			DataPtr = Elements.GetData() + InsertIndex;
			DestructItem(DataPtr);
		}
		else
		{
			// Adding new one, this may reallocate Elements
			Elements.InsertUninitialized(InsertIndex);
			DataPtr = Elements.GetData() + InsertIndex;
		}

		if (bIsAlreadyInSetPtr)
		{
			*bIsAlreadyInSetPtr = bAlreadyExists;
		}

		return DataPtr;
	}

	/** The base of TSortedSet iterators */
	template<bool bConst>
	class TBaseIterator
	{
	private:
		using ArrayIteratorType = std::conditional_t<bConst, typename ElementArrayType::TConstIterator, typename ElementArrayType::TIterator>;
		using ItElementType = std::conditional_t<bConst, const ElementType, ElementType>;

	protected:
		[[nodiscard]] UE_FORCEINLINE_HINT TBaseIterator(const ArrayIteratorType& InArrayIt)
			: ArrayIt(InArrayIt)
		{
		}

	public:
		UE_FORCEINLINE_HINT TBaseIterator& operator++()
		{
			++ArrayIt;
			return *this;
		}

		/** conversion to "bool" returning true if the iterator is valid. */
		[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
		{
			return !!ArrayIt; 
		}

		[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TBaseIterator& Rhs) const
		{
			return ArrayIt == Rhs.ArrayIt;
		}
		[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TBaseIterator& Rhs) const
		{
			return !(*this == Rhs);
		}

		[[nodiscard]] UE_FORCEINLINE_HINT FSetElementId GetId() const
		{
			return FSetElementId::FromInteger(ArrayIt.GetIndex());
		}

		UE_FORCEINLINE_HINT ItElementType& operator*() const
		{
			return *ArrayIt;
		}
		UE_FORCEINLINE_HINT ItElementType* operator->() const
		{
			return &*ArrayIt;
		}

	protected:
		ArrayIteratorType ArrayIt;
	};

	/** An array of the elements in the set */
	ElementArrayType Elements;

public:
	/** Set iterator */
	class TIterator : public TBaseIterator<false>
	{
	public:
		[[nodiscard]] UE_FORCEINLINE_HINT explicit TIterator(TSortedSet& InSet)
			: TBaseIterator<false>(InSet.Elements.CreateIterator())
		{
		}

		/** Removes the current element from the set */
		UE_FORCEINLINE_HINT void RemoveCurrent()
		{
			TBaseIterator<false>::ArrayIt.RemoveCurrent();
		}
	};

	/** Const set iterator */
	class TConstIterator : public TBaseIterator<true>
	{
	public:
		[[nodiscard]] UE_FORCEINLINE_HINT explicit TConstIterator(const TSortedSet& InSet)
			: TBaseIterator<true>(InSet.Elements.CreateConstIterator())
		{
		}
	};

	/** Creates an iterator over all the elements in this set */
	[[nodiscard]] UE_FORCEINLINE_HINT TIterator CreateIterator()
	{
		return TIterator(*this);
	}

	/** Creates a const iterator over all the elements in this set */
	[[nodiscard]] UE_FORCEINLINE_HINT TConstIterator CreateConstIterator() const
	{
		return TConstIterator(*this);
	}

public:
	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT auto begin()
	{
		return Elements.begin();
	}
	[[nodiscard]] UE_FORCEINLINE_HINT auto begin() const
	{
		return Elements.begin();
	}
	[[nodiscard]] UE_FORCEINLINE_HINT auto end()
	{
		return Elements.end();
	}
	[[nodiscard]] UE_FORCEINLINE_HINT auto end() const
	{
		return Elements.end();
	}
};

DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT((template <typename ElementType, typename ArrayAllocator, typename SortPredicate>), (TSortedSet<ElementType, ArrayAllocator, SortPredicate>));

template <typename ElementType, typename ArrayAllocator, typename SortPredicate>
UE_FORCEINLINE_HINT FArchive& operator<<(FArchive& Ar, TSortedSet<ElementType, ArrayAllocator, SortPredicate>& Set)
{
	Ar << Set.Elements;

	if (Ar.IsLoading())
	{
		// We need to re-sort, in case the predicate is not consistent with what it was before
		Algo::Sort(Set.Elements, SortPredicate());
	}

	return Ar;
}

