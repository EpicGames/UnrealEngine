// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Templates/Less.h"
#include "Containers/Array.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/ScriptArray.h"
#include "Containers/BitArray.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/MemoryImageWriter.h"
#include "Containers/UnrealString.h"
#include "Misc/IntrusiveUnsetOptionalState.h"
#include "Traits/IsTriviallyRelocatable.h"
#include <initializer_list>

#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	#define TSPARSEARRAY_RANGED_FOR_CHECKS 0
#else
	#define TSPARSEARRAY_RANGED_FOR_CHECKS 1
#endif

/** The result of a sparse array allocation. */
struct FSparseArrayAllocationInfo
{
	int32 Index;
	void* Pointer;
};

// Forward declarations.
template<typename ElementType,typename Allocator = FDefaultSparseArrayAllocator >
class TSparseArray;

template <typename T, typename Allocator> void* operator new(size_t Size, TSparseArray<T, Allocator>& Array);
template <typename T, typename Allocator> void* operator new(size_t Size, TSparseArray<T, Allocator>& Array, int32 Index);
inline void* operator new(size_t Size, const FSparseArrayAllocationInfo& Allocation);

/** Allocated elements are overlapped with free element info in the element list. */
template<typename ElementType>
union TSparseArrayElementOrFreeListLink
{
	/** If the element is allocated, its value is stored here. */
	ElementType ElementData;

	struct
	{
		/** If the element isn't allocated, this is a link to the previous element in the array's free list. */
		int32 PrevFreeIndex;

		/** If the element isn't allocated, this is a link to the next element in the array's free list. */
		int32 NextFreeIndex;
	};
};

DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT(template<typename ElementType>, TSparseArrayElementOrFreeListLink<ElementType>);

template<size_t SizeOfElementType, size_t AlignOfElementType, typename Allocator>
class TSparseArrayBase
{
protected:
	[[nodiscard]] explicit TSparseArrayBase(FIntrusiveUnsetOptionalState Tag)
		: Data(Tag)
	{
	}

	/** Default constructor. */
	[[nodiscard]] constexpr TSparseArrayBase()
		: FirstFreeIndex(-1)
		, NumFreeIndices(0)
	{
	}

	/** Explicitly consteval constructor. */
	[[nodiscard]] explicit consteval TSparseArrayBase(EConstEval)
		: Data(ConstEval)
		, AllocationFlags(ConstEval)
		, FirstFreeIndex(-1)
		, NumFreeIndices(0)
	{
	}

	/** Copy constructor. */
	[[nodiscard]] TSparseArrayBase(const TSparseArrayBase& InCopy)
		: FirstFreeIndex(-1)
		, NumFreeIndices(0)
	{
	}

	/** Marks an index as allocated, and returns information about the allocation. */
	FSparseArrayAllocationInfo AllocateIndex(int32 Index)
	{
		check(Index >= 0);
		check(Index < GetMaxIndex());
		check(!AllocationFlags[Index]);

		// Flag the element as allocated.
		AllocationFlags[Index] = true;

		// Set the allocation info.
		FSparseArrayAllocationInfo Result;
		Result.Index = Index;
		Result.Pointer = &((FElementOrFreeListLink*)Data.GetData())[Result.Index].ElementData;

		return Result;
	}

	/**
	 * Allocates space for an element in the array.  The element is not initialized, and you must use the corresponding placement new operator
	 * to construct the element in the allocated memory.
	 */
	FSparseArrayAllocationInfo AddUninitialized()
	{
		int32 Index;
		if(NumFreeIndices)
		{
			FElementOrFreeListLink* DataPtr = (FElementOrFreeListLink*)Data.GetData();

			// Remove and use the first index from the list of free elements.
			Index = FirstFreeIndex;
			FirstFreeIndex = DataPtr[FirstFreeIndex].NextFreeIndex;
			--NumFreeIndices;
			if(NumFreeIndices)
			{
				DataPtr[FirstFreeIndex].PrevFreeIndex = -1;
			}
		}
		else
		{
			// Add a new element.
			Index = Data.AddUninitialized(1);
			AllocationFlags.Add(false);
		}

		return AllocateIndex(Index);
	}

	FSparseArrayAllocationInfo AddUninitializedAtLowestFreeIndex(int32& LowestFreeIndexSearchStart)
	{
		FElementOrFreeListLink* DataPtr;

		int32 Index;
		if(NumFreeIndices)
		{
			Index = AllocationFlags.FindAndSetFirstZeroBit(LowestFreeIndexSearchStart);
			LowestFreeIndexSearchStart = Index + 1;

			DataPtr = (FElementOrFreeListLink*)Data.GetData();

			// Update FirstFreeIndex
			if (FirstFreeIndex == Index)
			{
				FirstFreeIndex = DataPtr[Index].NextFreeIndex;
			}

			// Link our next and prev free nodes together
			if (DataPtr[Index].NextFreeIndex >= 0)
			{
				DataPtr[DataPtr[Index].NextFreeIndex].PrevFreeIndex = DataPtr[Index].PrevFreeIndex;
			}

			if (DataPtr[Index].PrevFreeIndex >= 0)
			{
				DataPtr[DataPtr[Index].PrevFreeIndex].NextFreeIndex = DataPtr[Index].NextFreeIndex;
			}

			--NumFreeIndices;
		}
		else
		{
			// Add a new element.
			Index = Data.AddUninitialized(1);
			AllocationFlags.Add(true);

			// Defer getting the data pointer until after a possible reallocation
			DataPtr = (FElementOrFreeListLink*)Data.GetData();
		}

		FSparseArrayAllocationInfo Result;
		Result.Index = Index;
		Result.Pointer = &DataPtr[Result.Index].ElementData;
		return Result;
	}

	/** Removes Count elements from the array, starting from Index, without destructing them. */
	void RemoveAtUninitialized(int32 Index,int32 Count = 1)
	{
		FElementOrFreeListLink* DataPtr = (FElementOrFreeListLink*)Data.GetData();

		for (; Count; --Count)
		{
			check(AllocationFlags[Index]);

			// Mark the element as free and add it to the free element list.
			if(NumFreeIndices)
			{
				DataPtr[FirstFreeIndex].PrevFreeIndex = Index;
			}
			DataPtr[Index].PrevFreeIndex = -1;
			DataPtr[Index].NextFreeIndex = NumFreeIndices > 0 ? FirstFreeIndex : INDEX_NONE;
			FirstFreeIndex = Index;
			++NumFreeIndices;
			AllocationFlags[Index] = false;

			++Index;
		}
	}

	/**
	 * Preallocates enough memory to contain the specified number of elements.
	 *
	 * @param	ExpectedNumElements		the total number of elements that the array will have
	 */
	void Reserve(int32 ExpectedNumElements)
	{
		if (ExpectedNumElements > Data.Num())
		{
			const int32 ElementsToAdd = ExpectedNumElements - Data.Num();

			// allocate memory in the array itself - reserve first to ensure that resizing-for-growth doesn't occur
			Data.Reserve(ExpectedNumElements);
			int32 ElementIndex = Data.AddUninitialized(ElementsToAdd);

			FElementOrFreeListLink* DataPtr = (FElementOrFreeListLink*)Data.GetData();

			// now mark the new elements as free
			for (int32 FreeIndex = ExpectedNumElements - 1; FreeIndex >= ElementIndex; --FreeIndex)
			{
				if (NumFreeIndices)
				{
					DataPtr[FirstFreeIndex].PrevFreeIndex = FreeIndex;
				}
				DataPtr[FreeIndex].PrevFreeIndex = -1;
				DataPtr[FreeIndex].NextFreeIndex = NumFreeIndices > 0 ? FirstFreeIndex : INDEX_NONE;
				FirstFreeIndex = FreeIndex;
				++NumFreeIndices;
			}

			if (ElementsToAdd == ExpectedNumElements)
			{
				AllocationFlags.Init(false, ElementsToAdd);
			}
			else
			{
				AllocationFlags.Add(false, ElementsToAdd);
			}
		}
	}

	/** Shrinks the array's storage to avoid slack. */
	void Shrink()
	{
		// Determine the highest allocated index in the data array.
		int32 MaxAllocatedIndex = AllocationFlags.FindLast(true);

		const int32 FirstIndexToRemove = MaxAllocatedIndex + 1;
		if (FirstIndexToRemove < Data.Num())
		{
			if (NumFreeIndices > 0)
			{
				FElementOrFreeListLink* DataPtr = (FElementOrFreeListLink*)Data.GetData();

				// Look for elements in the free list that are in the memory to be freed.
				int32 FreeIndex = FirstFreeIndex;
				while (FreeIndex != INDEX_NONE)
				{
					if (FreeIndex >= FirstIndexToRemove)
					{
						const int32 PrevFreeIndex = DataPtr[FreeIndex].PrevFreeIndex;
						const int32 NextFreeIndex = DataPtr[FreeIndex].NextFreeIndex;
						if (NextFreeIndex != -1)
						{
							DataPtr[NextFreeIndex].PrevFreeIndex = PrevFreeIndex;
						}
						if (PrevFreeIndex != -1)
						{
							DataPtr[PrevFreeIndex].NextFreeIndex = NextFreeIndex;
						}
						else
						{
							FirstFreeIndex = NextFreeIndex;
						}
						--NumFreeIndices;

						FreeIndex = NextFreeIndex;
					}
					else
					{
						FreeIndex = DataPtr[FreeIndex].NextFreeIndex;
					}
				}
			}

			// Truncate unallocated elements at the end of the data array.
			Data.RemoveAt(FirstIndexToRemove, Data.Num() - FirstIndexToRemove, EAllowShrinking::No);
			AllocationFlags.RemoveAt(FirstIndexToRemove, AllocationFlags.Num() - FirstIndexToRemove);
		}

		// Shrink the data array.
		Data.Shrink();
	}

	/**
	* Sort the free element list so that subsequent allocations will occur in the lowest available
	* position resulting in tighter packing without moving any existing items. This also means
	* that assigned indices no longer depend on the order in which old items were removed, making
	* it easier to use the container when determinism is required (without a container reset).
	*
	* E.g., call SortFreeList() each frame to make the container assign the same indices when we
	* perform the following operations:
	*	Frame1: Add(A) -> [0]; Add(B) -> [1]; Remove(A); Remove(B); (Free list is now {[1],[0],...})
	*	Frame2: Add(A) -> [1]; Add(B) -> [0]; Remove(A); Remove(B); (Free list is now {[0],[1],...})
	*
	* NOTE: This is operation is currently O(N) with N = GetMaxIndex(). This could be improved for
	* large mostly-full arrays if necessary.
	*/
	void SortFreeList()
	{
		FElementOrFreeListLink* DataPtr = (FElementOrFreeListLink*)Data.GetData();
		int32 CurrentHeadIndex = INDEX_NONE;
		int32 NumFreeIndicesProcessed = 0;

		// Reverse iteration to build the list from low to high indices
		for (int32 Index = Data.Num() - 1; NumFreeIndicesProcessed < NumFreeIndices; --Index)
		{
			// If we have an unused element, add it to the free list
			if (!IsValidIndex(Index))
			{
				DataPtr[Index].PrevFreeIndex = INDEX_NONE;
				DataPtr[Index].NextFreeIndex = INDEX_NONE;

				if (CurrentHeadIndex != INDEX_NONE)
				{
					DataPtr[CurrentHeadIndex].PrevFreeIndex = Index;
					DataPtr[Index].NextFreeIndex = CurrentHeadIndex;
				}

				CurrentHeadIndex = Index;
				++NumFreeIndicesProcessed;
			}
		}

		// Set up the new head
		FirstFreeIndex = CurrentHeadIndex;
	}

	/**
	 * Allocates space for an element in the array at a given index.  The element is not initialized, and you must use the corresponding placement new operator
	 * to construct the element in the allocated memory.
	 */
	FSparseArrayAllocationInfo InsertUninitialized(int32 Index)
	{
		FElementOrFreeListLink* DataPtr;

		// Enlarge the array to include the given index.
		if (Index >= Data.Num())
		{
			Data.AddUninitialized(Index + 1 - Data.Num());

			// Defer getting the data pointer until after a possible reallocation
			DataPtr = (FElementOrFreeListLink*)Data.GetData();

			while (AllocationFlags.Num() < Data.Num())
			{
				const int32 FreeIndex = AllocationFlags.Num();
				DataPtr[FreeIndex].PrevFreeIndex = -1;
				DataPtr[FreeIndex].NextFreeIndex = FirstFreeIndex;
				if (NumFreeIndices)
				{
					DataPtr[FirstFreeIndex].PrevFreeIndex = FreeIndex;
				}
				FirstFreeIndex = FreeIndex;
				verify(AllocationFlags.Add(false) == FreeIndex);
				++NumFreeIndices;
			};
		}
		else
		{
			DataPtr = (FElementOrFreeListLink*)Data.GetData();
		}

		// Verify that the specified index is free.
		check(!AllocationFlags[Index]);

		// Remove the index from the list of free elements.
		--NumFreeIndices;
		const int32 PrevFreeIndex = DataPtr[Index].PrevFreeIndex;
		const int32 NextFreeIndex = DataPtr[Index].NextFreeIndex;
		if (PrevFreeIndex != -1)
		{
			DataPtr[PrevFreeIndex].NextFreeIndex = NextFreeIndex;
		}
		else
		{
			FirstFreeIndex = NextFreeIndex;
		}
		if (NextFreeIndex != -1)
		{
			DataPtr[NextFreeIndex].PrevFreeIndex = PrevFreeIndex;
		}

		return AllocateIndex(Index);
	}

	/** Compacts the allocated elements into a contiguous index range. */
	/** Returns true if any elements were relocated, false otherwise. */
	bool Compact()
	{
		int32 NumFree = NumFreeIndices;
		if (NumFree == 0)
		{
			return false;
		}

		bool bResult = false;

		FElementOrFreeListLink* DataPtr = (FElementOrFreeListLink*)Data.GetData();

		int32 EndIndex    = Data.Num();
		int32 TargetIndex = EndIndex - NumFree;
		int32 FreeIndex   = FirstFreeIndex;
		while (FreeIndex != -1)
		{
			int32 NextFreeIndex = DataPtr[FreeIndex].NextFreeIndex;
			if (FreeIndex < TargetIndex)
			{
				// We need an element here
				do
				{
					--EndIndex;
				}
				while (!AllocationFlags[EndIndex]);

				RelocateConstructItems<FElementOrFreeListLink>(DataPtr + FreeIndex, DataPtr + EndIndex, 1);
				AllocationFlags[FreeIndex] = true;

				bResult = true;
			}

			FreeIndex = NextFreeIndex;
		}

		Data.RemoveAt(TargetIndex, NumFree, EAllowShrinking::No);
		AllocationFlags.RemoveAt(TargetIndex, NumFree);

		NumFreeIndices = 0;
		FirstFreeIndex = -1;

		// Shrink the data array.
		Data.Shrink();

		return bResult;
	}

	[[nodiscard]] int32 GetMaxIndex() const
	{
		return Data.Num();
	}

	[[nodiscard]] bool IsEmpty() const
	{
		return Data.Num() == NumFreeIndices;
	}

	[[nodiscard]] int32 Num() const
	{
		return Data.Num() - NumFreeIndices;
	}

	/** @return The number of elements the array can hold before reallocation. */
	[[nodiscard]] int32 Max() const
	{
		return Data.Max();
	}

	[[nodiscard]] bool IsValidIndex(int32 Index) const
	{
		return AllocationFlags.IsValidIndex(Index) && AllocationFlags[Index];
	}

	[[nodiscard]] bool IsAllocated(int32 Index) const
	{
		return AllocationFlags[Index];
	}

	/**
	 * The element type stored is only indirectly related to the element type requested, to avoid instantiating TArray redundantly for
	 * compatible types.
	 */
	typedef TSparseArrayElementOrFreeListLink<
		TAlignedBytes<SizeOfElementType, AlignOfElementType>
		> FElementOrFreeListLink;

	typedef TArray<FElementOrFreeListLink, typename Allocator::ElementAllocator> DataType;
	DataType Data;

	typedef TBitArray<typename Allocator::BitArrayAllocator> AllocationBitArrayType;
	AllocationBitArrayType AllocationFlags;

	/** The index of an unallocated element in the array that currently contains the head of the linked list of free elements. */
	int32 FirstFreeIndex = -1;

	/** The number of elements in the free list. */
	int32 NumFreeIndices = 0;
};

template <typename AllocatorType, typename InDerivedType = void>
class TScriptSparseArray;

/**
 * A dynamically sized array where element indices aren't necessarily contiguous.  Memory is allocated for all
 * elements in the array's index range, so it doesn't save memory; but it does allow O(1) element removal that
 * doesn't invalidate the indices of subsequent elements.  It uses TArray to store the elements, and a TBitArray
 * to store whether each element index is allocated (for fast iteration over allocated elements).
 *
 **/
template<typename InElementType, typename Allocator /*= FDefaultSparseArrayAllocator */>
class TSparseArray : public TSparseArrayBase<sizeof(InElementType), alignof(InElementType), Allocator>
{
	using SuperType = TSparseArrayBase<sizeof(InElementType), alignof(InElementType), Allocator>;
	using ElementType = InElementType;

	template <typename, typename>
	friend class TScriptSparseArray;

public:
	/** Destructor. */
	~TSparseArray()
	{
		UE_STATIC_ASSERT_WARN(TIsTriviallyRelocatable_V<InElementType>, "TArray can only be used with trivially relocatable types");

		// Destruct the elements in the array.
		Empty();
	}

	/////////////////////////////////////////////////////
	// Start - intrusive TOptional<TSparseArray> state //
	/////////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TSparseArray;

	[[nodiscard]] explicit TSparseArray(FIntrusiveUnsetOptionalState Tag)
		: SuperType(Tag)
	{
	}

	[[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState Tag) const
	{
		return Data == Tag;
	}
	///////////////////////////////////////////////////
	// End - intrusive TOptional<TSparseArray> state //
	///////////////////////////////////////////////////

	using SuperType::AllocateIndex;
	using SuperType::AddUninitialized;

	/** Adds an element to the array. */
	int32 Add(const ElementType& Element)
	{
		FSparseArrayAllocationInfo Allocation = AddUninitialized();
		new(Allocation) ElementType(Element);
		return Allocation.Index;
	}

	/** Adds an element to the array. */
	int32 Add(ElementType&& Element)
	{
		FSparseArrayAllocationInfo Allocation = AddUninitialized();
		new(Allocation) ElementType(MoveTemp(Element));
		return Allocation.Index;
	}

	using SuperType::AddUninitializedAtLowestFreeIndex;

	/**
	 * Constructs a new item at the last freed index of the array.
	 *
	 * @param Args	The arguments to forward to the constructor of the new item.
	 * @return		Index to the new item
	 */
	template <typename... ArgsType>
	inline int32 Emplace(ArgsType&&... Args)
	{
		FSparseArrayAllocationInfo Allocation = AddUninitialized();
		new(Allocation) ElementType(Forward<ArgsType>(Args)...);
		return Allocation.Index;
	}

	/**
	 * Constructs a new item at the lowest free index of the array.
	 * This requires a search which can be accelerated with LowestFreeIndexSearchStart.
	 *
	 * @param LowestFreeIndexSearchStart	Where to start the search for a free index.
	 * @param Args							The arguments to forward to the constructor of the new item.
	 * @return								Index to the new item
	 */
	template <typename... ArgsType>
	inline int32 EmplaceAtLowestFreeIndex(int32& LowestFreeIndexSearchStart, ArgsType&&... Args)
	{
		FSparseArrayAllocationInfo Allocation = AddUninitializedAtLowestFreeIndex(LowestFreeIndexSearchStart);
		new(Allocation) ElementType(Forward<ArgsType>(Args)...);
		return Allocation.Index;
	}

	/**
	 * Constructs a new item at a given index of the array.
	 *
	 * @param Index							Index at which the new allocation will be done
	 * @param Args							The arguments to forward to the constructor of the new item.
	 * @return								Index to the new item
	 */
	template <typename... ArgsType>
	inline int32 EmplaceAt(int32 Index, ArgsType&&... Args)
	{
		FSparseArrayAllocationInfo Allocation;
		if(!AllocationFlags.IsValidIndex(Index) || !AllocationFlags[Index])
		{
			Allocation = InsertUninitialized(Index);
		}
		else
		{
			FElementOrFreeListLink* Elem = (FElementOrFreeListLink*)Data.GetData() + Index;
			((ElementType&)Elem->ElementData).~ElementType();

			Allocation.Index = Index;
			Allocation.Pointer = &Elem->ElementData;
		}

		new(Allocation) ElementType(Forward<ArgsType>(Args)...);
		return Allocation.Index;
	}

	using SuperType::InsertUninitialized;

	/**
	 * Inserts an element to the array.
	 */
	void Insert(int32 Index,typename TTypeTraits<ElementType>::ConstInitType Element)
	{
		new(InsertUninitialized(Index)) ElementType(Element);
	}

	/** Removes Count elements from the array, starting from Index. */
	void RemoveAt(int32 Index,int32 Count = 1)
	{
		if constexpr (!std::is_trivially_destructible_v<ElementType>)
		{
			DestroyElements(Index, Count);
		}

		RemoveAtUninitialized(Index, Count);
	}

	using SuperType::RemoveAtUninitialized;

	/**
	 * Removes all elements from the array, potentially leaving space allocated for an expected number of elements about to be added.
	 * @param ExpectedNumElements - The expected number of elements about to be added.
	 */
	void Empty(int32 ExpectedNumElements = 0)
	{
		// Destruct the allocated elements.
		if constexpr (!std::is_trivially_destructible_v<ElementType>)
		{
			DestroyElements();
		}

		// Free the allocated elements.
		Data.Empty(ExpectedNumElements);
		FirstFreeIndex = -1;
		NumFreeIndices = 0;
		AllocationFlags.Empty(ExpectedNumElements);
	}

	/** Empties the array, but keep its allocated memory as slack. */
	void Reset()
	{
		// Destruct the allocated elements.
		if constexpr (!std::is_trivially_destructible_v<ElementType>)
		{
			DestroyElements();
		}

		// Free the allocated elements.
		Data.Reset();
		FirstFreeIndex = -1;
		NumFreeIndices = 0;
		AllocationFlags.Reset();
	}

	using SuperType::Reserve;
	using SuperType::Shrink;
	using SuperType::Compact;

	/** Compacts the allocated elements into a contiguous index range. Does not change the iteration order of the elements. */
	/** Returns true if any elements were relocated, false otherwise. */
	bool CompactStable()
	{
		if (NumFreeIndices == 0)
		{
			return false;
		}

		// Copy the existing elements to a new array.
		TSparseArray<ElementType,Allocator> CompactedArray;
		CompactedArray.Empty(Num());
		for(TIterator It(*this);It;++It)
		{
			new(CompactedArray.AddUninitialized()) ElementType(MoveTempIfPossible(*It));
		}

		// Replace this array with the compacted array.
		Exchange(*this,CompactedArray);

		return true;
	}

	/** Sorts the elements using the provided comparison class. */
	template<typename PREDICATE_CLASS>
	void Sort( const PREDICATE_CLASS& Predicate )
	{
		if(Num() > 0)
		{
			// Compact the elements array so all the elements are contiguous.
			Compact();

			// Sort the elements according to the provided comparison class.
			Algo::Sort(TArrayView<FElementOrFreeListLink>(Data.GetData(), Num()), FElementCompareClass< PREDICATE_CLASS >( Predicate ) );
		}
	}

	/** Sorts the elements assuming < operator is defined for ElementType. */
	void Sort()
	{
		Sort( TLess< ElementType >() );
	}

	/** Stable sorts the elements using the provided comparison class. */
	template<typename PREDICATE_CLASS>
	void StableSort(const PREDICATE_CLASS& Predicate)
	{
		if (Num() > 0)
		{
			// Compact the elements array so all the elements are contiguous.
			CompactStable();

			// Sort the elements according to the provided comparison class.
			Algo::StableSort(TArrayView<FElementOrFreeListLink>(Data.GetData(), Num()), FElementCompareClass< PREDICATE_CLASS >(Predicate));
		}
	}

	/** Stable sorts the elements assuming < operator is defined for ElementType. */
	void StableSort()
	{
		StableSort(TLess< ElementType >());
	}

	using SuperType::SortFreeList;

	/**
	 * Finds an item by predicate.
	 *
	 * @param Pred The predicate to match.
	 * @returns Index to the first matching element, or INDEX_NONE if none is found.
	 */
	template <typename Predicate>
	[[nodiscard]] int32 IndexOfByPredicate(Predicate Pred) const
	{
		for (TConstIterator It = CreateConstIterator(); It; ++It)
		{
			if (::Invoke(Pred, *It))
			{
				return It.GetIndex();
			}
		}

		return INDEX_NONE;
	}

	/**
	 * Finds the index of an arbitrary element.
	 * Callers should not depend on particular patterns in the behaviour of this function.
	 * @return The index of any element in the container, or INDEX_NONE if it is empty.
	 */
	[[nodiscard]] int32 FindArbitraryElementIndex() const
	{
		// The goal of this function is to be fast, and so the implementation may be improved at any time even if it gives different results.

		if (NumFreeIndices == 0)
		{
			return Data.Num() - 1; // this will return INDEX_NONE if Data is empty.
		}

		return AllocationFlags.Find(true);
	}

	/**
	 * Helper function to return the amount of memory allocated by this container
	 * Only returns the size of allocations made directly by the container, not the elements themselves.
	 * @return number of bytes allocated by this container
	 */
	[[nodiscard]] SIZE_T GetAllocatedSize( void ) const
	{
		return Data.GetAllocatedSize() + AllocationFlags.GetAllocatedSize();
	}

	/** Tracks the container's memory use through an archive. */
	void CountBytes(FArchive& Ar) const
	{
		Data.CountBytes(Ar);
		AllocationFlags.CountBytes(Ar);
	}

	[[nodiscard]] bool IsCompact() const
	{
		return NumFreeIndices == 0;
	}

	/**
	 * Equality comparison operator.
	 * Checks that both arrays have the same elements and element indices; that means that unallocated elements are signifigant!
	 */
	[[nodiscard]] bool operator==(const TSparseArray& B) const
	{
		if (GetMaxIndex() != B.GetMaxIndex())
		{
			return false;
		}

		for (int32 ElementIndex = 0; ElementIndex < GetMaxIndex(); ElementIndex++)
		{
			const bool bIsAllocatedA = IsAllocated(ElementIndex);
			const bool bIsAllocatedB = B.IsAllocated(ElementIndex);
			if (bIsAllocatedA != bIsAllocatedB)
			{
				return false;
			}
			else if (bIsAllocatedA)
			{
				if ((*this)[ElementIndex] != B[ElementIndex])
				{
					return false;
				}
			}
		}

		return true;
	}

	/**
	 * Inequality comparison operator.
	 * Checks that both arrays have the same elements and element indices; that means that unallocated elements are signifigant!
	 */
	[[nodiscard]] bool operator!=(const TSparseArray& B) const
	{
		return !(*this == B);
	}

	/** Default constructor. */
	[[nodiscard]] constexpr TSparseArray() = default;

	[[nodiscard]] explicit consteval TSparseArray(EConstEval)
		: SuperType(ConstEval)
	{
	}

	/** Move constructor. */
	[[nodiscard]] TSparseArray(TSparseArray&& InCopy)
	{
		this->Move(*this, InCopy);
	}

	/** Copy constructor. */
	[[nodiscard]] TSparseArray(const TSparseArray& InCopy)
		: SuperType(InCopy)
	{
		*this = InCopy;
	}

	/** Initializer list constructor. */
	[[nodiscard]] TSparseArray(std::initializer_list<ElementType> InList)
		: SuperType()
	{
		*this = InList;
	}

	/** Move assignment operator. */
	TSparseArray& operator=(TSparseArray&& InCopy)
	{
		if(this != &InCopy)
		{
			this->Move(*this, InCopy);
		}
		return *this;
	}

	/** Copy assignment operator. */
	TSparseArray& operator=(const TSparseArray& InCopy)
	{
		if (this != &InCopy)
		{
			int32 SrcMax = InCopy.GetMaxIndex();

			// Reallocate the array.
			Empty(SrcMax);
			Data.AddUninitialized(SrcMax);

			// Copy the other array's element allocation state.
			FirstFreeIndex  = InCopy.FirstFreeIndex;
			NumFreeIndices  = InCopy.NumFreeIndices;
			AllocationFlags = InCopy.AllocationFlags;

			      FElementOrFreeListLink* DestData = (      FElementOrFreeListLink*)Data.GetData();
			const FElementOrFreeListLink* SrcData  = (const FElementOrFreeListLink*)InCopy.Data.GetData();

			// Determine whether we need per element construction or bulk copy is fine
			if constexpr (!std::is_trivially_copy_constructible_v<ElementType>)
			{
				// Use the inplace new to copy the element to an array element
				for (int32 Index = 0; Index < SrcMax; ++Index)
				{
					      FElementOrFreeListLink& DestElement = DestData[Index];
					const FElementOrFreeListLink& SrcElement  = SrcData [Index];
					if (InCopy.IsAllocated(Index))
					{
						::new((uint8*)&DestElement.ElementData) ElementType(*(const ElementType*)&SrcElement.ElementData);
					}
					else
					{
						DestElement.PrevFreeIndex = SrcElement.PrevFreeIndex;
						DestElement.NextFreeIndex = SrcElement.NextFreeIndex;
					}
				}
			}
			else
			{
				if (SrcMax)
				{
					// Use the much faster path for types that allow it
					FMemory::Memcpy(DestData, SrcData, sizeof(FElementOrFreeListLink) * SrcMax);
				}
			}
		}
		return *this;
	}

	/** Initializer list assignment operator. */
	TSparseArray& operator=(std::initializer_list<ElementType> InList)
	{
		this->Reset();
		this->Reserve((uint32)InList.size());
		for (const ElementType& Element : InList)
		{
			Add(Element);
		}
		return *this;
	}

private:
	template <typename SparseArrayType>
	inline static void Move(SparseArrayType& ToArray, SparseArrayType& FromArray)
	{
		// Destruct the allocated elements.
		if constexpr (!std::is_trivially_destructible_v<ElementType>)
		{
			ToArray.DestroyElements();
		}

		ToArray.Data            = (DataType&&)FromArray.Data;
		ToArray.AllocationFlags = (AllocationBitArrayType&&)FromArray.AllocationFlags;

		ToArray.FirstFreeIndex = FromArray.FirstFreeIndex;
		ToArray.NumFreeIndices = FromArray.NumFreeIndices;
		FromArray.FirstFreeIndex = -1;
		FromArray.NumFreeIndices = 0;
	}

	FORCEINLINE void DestroyElements()
	{
		DestroyElements(0, Data.Num());
	}

	FORCENOINLINE void DestroyElements(int32 Index, int32 Count)
	{
		const int32 EndIndex = Index + Count;
		FElementOrFreeListLink* DataPtr = (FElementOrFreeListLink*)Data.GetData();
		for (int32 i = Index; i < EndIndex; ++i)
		{
			if (IsValidIndex(i))
			{
				((ElementType&)DataPtr[i].ElementData).~ElementType();
			}
		}
	}

public:
	// Accessors.
	[[nodiscard]] ElementType& operator[](int32 Index)
	{
		checkSlow(Index >= 0 && Index < Data.Num() && Index < AllocationFlags.Num());
		//checkSlow(AllocationFlags[Index]); // Disabled to improve loading times -BZ
		return *(ElementType*)&((FElementOrFreeListLink*)Data.GetData())[Index].ElementData;
	}
	[[nodiscard]] const ElementType& operator[](int32 Index) const
	{
		checkSlow(Index >= 0 && Index < Data.Num() && Index < AllocationFlags.Num());
		//checkSlow(AllocationFlags[Index]); // Disabled to improve loading times -BZ
		return *(ElementType*)&((FElementOrFreeListLink*)Data.GetData())[Index].ElementData;
	}
	[[nodiscard]] int32 PointerToIndex(const ElementType* Ptr) const
	{
		checkSlow(Data.Num());
		int32 Index = (int32)((FElementOrFreeListLink*)Ptr - (FElementOrFreeListLink*)Data.GetData());
		checkSlow(Index >= 0 && Index < Data.Num() && Index < AllocationFlags.Num() && AllocationFlags[Index]);
		return Index;
	}
	using SuperType::IsValidIndex;
	using SuperType::IsAllocated;
	using SuperType::GetMaxIndex;
	using SuperType::IsEmpty;
	using SuperType::Num;
	using SuperType::Max;

	/**
	 * Checks that the specified address is not part of an element within the container.  Used for implementations
	 * to check that reference arguments aren't going to be invalidated by possible reallocation.
	 *
	 * @param Addr The address to check.
	 */
	UE_FORCEINLINE_HINT void CheckAddress(const ElementType* Addr) const
	{
		Data.CheckAddress(Addr);
	}

private:

	/** The base class of sparse array iterators. */
	template<bool bConst>
	class TBaseIterator
	{
	public:
		typedef TConstSetBitIterator<typename Allocator::BitArrayAllocator> BitArrayItType;

	private:
		typedef std::conditional_t<bConst,const TSparseArray,TSparseArray> ArrayType;
		typedef std::conditional_t<bConst,const ElementType,ElementType> ItElementType;

	public:
		[[nodiscard]] explicit TBaseIterator(ArrayType& InArray, const BitArrayItType& InBitArrayIt)
			: Array     (InArray)
			, BitArrayIt(InBitArrayIt)
		{
		}

		inline TBaseIterator& operator++()
		{
			// Iterate to the next set allocation flag.
			++BitArrayIt;
			return *this;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT int32 GetIndex() const
		{
			return BitArrayIt.GetIndex();
		}

		[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TBaseIterator& Rhs) const
		{
			return BitArrayIt == Rhs.BitArrayIt && &Array == &Rhs.Array;
		}
		[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TBaseIterator& Rhs) const
		{
			return BitArrayIt != Rhs.BitArrayIt || &Array != &Rhs.Array;
		}

		/** conversion to "bool" returning true if the iterator is valid. */
		[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
		{
			return !!BitArrayIt;
		}

		/** inverse of the "bool" operator */
		[[nodiscard]] UE_FORCEINLINE_HINT bool operator !() const
		{
			return !(bool)*this;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT ItElementType& operator*() const
		{
			return Array[GetIndex()];
		}
		[[nodiscard]] UE_FORCEINLINE_HINT ItElementType* operator->() const
		{
			return &Array[GetIndex()];
		}
		[[nodiscard]] UE_FORCEINLINE_HINT const FRelativeBitReference& GetRelativeBitReference() const
		{
			return BitArrayIt;
		}

	protected:
		ArrayType&     Array;
		BitArrayItType BitArrayIt;
	};

public:

	/** Iterates over all allocated elements in a sparse array. */
	class TIterator : public TBaseIterator<false>
	{
	public:
		[[nodiscard]] TIterator(TSparseArray& InArray)
			: TBaseIterator<false>(InArray, TConstSetBitIterator<typename Allocator::BitArrayAllocator>(InArray.AllocationFlags))
		{
		}

		[[nodiscard]] TIterator(TSparseArray& InArray, const typename TBaseIterator<false>::BitArrayItType& InBitArrayIt)
			: TBaseIterator<false>(InArray, InBitArrayIt)
		{
		}

		/** Safely removes the current element from the array. */
		void RemoveCurrent()
		{
			this->Array.RemoveAt(this->GetIndex());
		}
	};

	/** Iterates over all allocated elements in a const sparse array. */
	class TConstIterator : public TBaseIterator<true>
	{
	public:
		[[nodiscard]] TConstIterator(const TSparseArray& InArray)
			: TBaseIterator<true>(InArray, TConstSetBitIterator<typename Allocator::BitArrayAllocator>(InArray.AllocationFlags))
		{
		}

		[[nodiscard]] TConstIterator(const TSparseArray& InArray, const typename TBaseIterator<true>::BitArrayItType& InBitArrayIt)
			: TBaseIterator<true>(InArray, InBitArrayIt)
		{
		}
	};

	#if TSPARSEARRAY_RANGED_FOR_CHECKS
		class TRangedForIterator : public TIterator
		{
		public:
			[[nodiscard]] TRangedForIterator(TSparseArray& InArray, const typename TBaseIterator<false>::BitArrayItType& InBitArrayIt)
				: TIterator (InArray, InBitArrayIt)
				, InitialNum(InArray.Num())
			{
			}

			[[nodiscard]] inline bool operator!=(const TRangedForIterator& Rhs) const
			{
				// We only need to do the check in this operator, because no other operator will be
				// called until after this one returns.
				//
				// Also, we should only need to check one side of this comparison - if the other iterator isn't
				// even from the same array then the compiler has generated bad code.
				ensureMsgf(this->Array.Num() == InitialNum, TEXT("Container has changed during ranged-for iteration!"));
				return *(TIterator*)this != *(TIterator*)&Rhs;
			}
		private:
			int32 InitialNum;
		};

		class TRangedForConstIterator : public TConstIterator
		{
		public:
			[[nodiscard]] TRangedForConstIterator(const TSparseArray& InArray, const typename TBaseIterator<true>::BitArrayItType& InBitArrayIt)
				: TConstIterator(InArray, InBitArrayIt)
				, InitialNum    (InArray.Num())
			{
			}

			[[nodiscard]] inline bool operator!=(const TRangedForConstIterator& Rhs) const
			{
				// We only need to do the check in this operator, because no other operator will be
				// called until after this one returns.
				//
				// Also, we should only need to check one side of this comparison - if the other iterator isn't
				// even from the same array then the compiler has generated bad code.
				ensureMsgf(this->Array.Num() == InitialNum, TEXT("Container has changed during ranged-for iteration!"));
				return *(TIterator*)this != *(TIterator*)&Rhs;
			}
		private:
			int32 InitialNum;
		};
	#else
		using TRangedForIterator      = TIterator;
		using TRangedForConstIterator = TConstIterator;
	#endif

	/** Creates an iterator for the contents of this array */
	[[nodiscard]] TIterator CreateIterator()
	{
		return TIterator(*this);
	}

	/** Creates a const iterator for the contents of this array */
	[[nodiscard]] TConstIterator CreateConstIterator() const
	{
		return TConstIterator(*this);
	}

public:
	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForIterator      begin()       { return TRangedForIterator     (*this, TConstSetBitIterator<typename Allocator::BitArrayAllocator>(AllocationFlags)); }
	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForConstIterator begin() const { return TRangedForConstIterator(*this, TConstSetBitIterator<typename Allocator::BitArrayAllocator>(AllocationFlags)); }
	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForIterator      end  ()       { return TRangedForIterator     (*this, TConstSetBitIterator<typename Allocator::BitArrayAllocator>(AllocationFlags, AllocationFlags.Num())); }
	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForConstIterator end  () const { return TRangedForConstIterator(*this, TConstSetBitIterator<typename Allocator::BitArrayAllocator>(AllocationFlags, AllocationFlags.Num())); }

public:
	/** An iterator which only iterates over the elements of the array which correspond to set bits in a separate bit array. */
	template<typename SubsetAllocator = FDefaultBitArrayAllocator>
	class TConstSubsetIterator
	{
	public:
		[[nodiscard]] TConstSubsetIterator( const TSparseArray& InArray, const TBitArray<SubsetAllocator>& InBitArray ):
			Array(InArray),
			BitArrayIt(InArray.AllocationFlags,InBitArray)
		{
		}
		inline TConstSubsetIterator& operator++()
		{
			// Iterate to the next element which is both allocated and has its bit set in the other bit array.
			++BitArrayIt;
			return *this;
		}
		[[nodiscard]] UE_FORCEINLINE_HINT int32 GetIndex() const
		{
			return BitArrayIt.GetIndex();
		}

		/** conversion to "bool" returning true if the iterator is valid. */
		[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
		{
			return !!BitArrayIt;
		}
		/** inverse of the "bool" operator */
		[[nodiscard]] UE_FORCEINLINE_HINT bool operator !() const
		{
			return !(bool)*this;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT const ElementType& operator*() const
		{
			return Array[GetIndex()];
		}
		[[nodiscard]] UE_FORCEINLINE_HINT const ElementType* operator->() const
		{
			return &Array[GetIndex()];
		}
		[[nodiscard]] UE_FORCEINLINE_HINT const FRelativeBitReference& GetRelativeBitReference() const
		{
			return BitArrayIt;
		}
	private:
		const TSparseArray& Array;
		TConstDualSetBitIterator<typename Allocator::BitArrayAllocator,SubsetAllocator> BitArrayIt;
	};

	/** Concatenation operators */
	TSparseArray& operator+=( const TSparseArray& OtherArray )
	{
		this->Reserve(this->Num() + OtherArray.Num());
		for ( typename TSparseArray::TConstIterator It(OtherArray); It; ++It )
		{
			this->Add(*It);
		}
		return *this;
	}
	TSparseArray& operator+=( const TArray<ElementType>& OtherArray )
	{
		this->Reserve(this->Num() + OtherArray.Num());
		for ( int32 Idx = 0; Idx < OtherArray.Num(); Idx++ )
		{
			this->Add(OtherArray[Idx]);
		}
		return *this;
	}

private:

	using FElementOrFreeListLink = typename SuperType::FElementOrFreeListLink;

	/** Extracts the element value from the array's element structure and passes it to the user provided comparison class. */
	template <typename PREDICATE_CLASS>
	class FElementCompareClass
	{
		const PREDICATE_CLASS& Predicate;

	public:
		[[nodiscard]] FElementCompareClass( const PREDICATE_CLASS& InPredicate )
			: Predicate( InPredicate )
		{
		}

		[[nodiscard]] bool operator()( const FElementOrFreeListLink& A,const FElementOrFreeListLink& B ) const
		{
			return Predicate(*(ElementType*)&A.ElementData,*(ElementType*)&B.ElementData);
		}
	};

	using typename SuperType::DataType;
	using typename SuperType::AllocationBitArrayType;

	using SuperType::Data;
	using SuperType::AllocationFlags;
	using SuperType::FirstFreeIndex;
	using SuperType::NumFreeIndices;

public:
	void WriteMemoryImage(FMemoryImageWriter& Writer) const
	{
		checkf(!Writer.Is32BitTarget(), TEXT("TSparseArray does not currently support freezing for 32bits"));
		if constexpr (TAllocatorTraits<Allocator>::SupportsFreezeMemoryImage && THasTypeLayout<ElementType>::Value)
		{
			// Write Data
			const int32 NumElements = this->Data.Num();
			if (NumElements > 0)
			{
				const FTypeLayoutDesc& ElementTypeDesc = StaticGetTypeLayoutDesc<ElementType>();
				FMemoryImageWriter ArrayWriter = Writer.WritePointer(ElementTypeDesc);
				for (int32 i = 0; i < NumElements; ++i)
				{
					const FElementOrFreeListLink& Elem = ((const FElementOrFreeListLink*)this->Data.GetData())[i];
					const uint32 StartOffset = ArrayWriter.WriteAlignment<FElementOrFreeListLink>();
					if (this->AllocationFlags[i])
					{
						ArrayWriter.WriteObject(&Elem.ElementData, ElementTypeDesc);
					}
					else
					{
						ArrayWriter.WriteBytes(Elem.PrevFreeIndex);
						ArrayWriter.WriteBytes(Elem.NextFreeIndex);
					}
					ArrayWriter.WritePaddingToSize(StartOffset + sizeof(FElementOrFreeListLink));
				}
			}
			else
			{
				Writer.WriteNullPointer();
			}
			Writer.WriteBytes(NumElements);
			Writer.WriteBytes(NumElements);

			//
			this->AllocationFlags.WriteMemoryImage(Writer);
			Writer.WriteBytes(this->FirstFreeIndex);
			Writer.WriteBytes(this->NumFreeIndices);
		}
		else
		{
			Writer.WriteBytes(TSparseArray());
		}
	}

	void CopyUnfrozen(const FMemoryUnfreezeContent& Context, void* Dst) const
	{
		if constexpr (TAllocatorTraits<Allocator>::SupportsFreezeMemoryImage && THasTypeLayout<ElementType>::Value)
		{
			const FTypeLayoutDesc& ElementTypeDesc = StaticGetTypeLayoutDesc<ElementType>();
			TSparseArray* DstObject = (TSparseArray*)Dst;
			{
				::new((void*)&DstObject->Data) DataType();
				DstObject->Data.SetNumUninitialized(this->Data.Num());
				for (int32 i = 0; i < this->Data.Num(); ++i)
				{
					const FElementOrFreeListLink& Elem    = ((const FElementOrFreeListLink*)this     ->Data.GetData())[i];
					      FElementOrFreeListLink& DstElem = ((      FElementOrFreeListLink*)DstObject->Data.GetData())[i];
					if (this->AllocationFlags[i])
					{
						Context.UnfreezeObject(&Elem.ElementData, ElementTypeDesc, &DstElem.ElementData);
					}
					else
					{
						DstElem.PrevFreeIndex = Elem.PrevFreeIndex;
						DstElem.NextFreeIndex = Elem.NextFreeIndex;
					}
				}
			}

			::new((void*)&DstObject->AllocationFlags) AllocationBitArrayType(this->AllocationFlags);
			DstObject->FirstFreeIndex = this->FirstFreeIndex;
			DstObject->NumFreeIndices = this->NumFreeIndices;
		}
		else
		{
			::new((void*)Dst) TSparseArray();
		}
	}

	static void AppendHash(const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		if constexpr (TAllocatorTraits<Allocator>::SupportsFreezeMemoryImage && THasTypeLayout<ElementType>::Value)
		{
			Freeze::AppendHash(StaticGetTypeLayoutDesc<ElementType>(), LayoutParams, Hasher);
		}
	}
};

namespace Freeze
{
	template<typename ElementType, typename Allocator>
	void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const TSparseArray<ElementType, Allocator>& Object, const FTypeLayoutDesc&)
	{
		Object.WriteMemoryImage(Writer);
	}

	template<typename ElementType, typename Allocator>
	uint32 IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const TSparseArray<ElementType, Allocator>& Object, void* OutDst)
	{
		Object.CopyUnfrozen(Context, OutDst);
		return sizeof(Object);
	}

	template<typename ElementType, typename Allocator>
	uint32 IntrinsicAppendHash(const TSparseArray<ElementType, Allocator>* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		TSparseArray<ElementType, Allocator>::AppendHash(LayoutParams, Hasher);
		return DefaultAppendHash(TypeDesc, LayoutParams, Hasher);
	}
}

DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT((template <typename ElementType, typename Allocator>), (TSparseArray<ElementType, Allocator>));

//
// TSparseArray operator news.
//
template <typename T, typename Allocator> void* operator new( size_t Size, TSparseArray<T, Allocator>& Array )
{
	check( Size == sizeof( T ) );
	const int32 Index = Array.AddUninitialized().Index;
	return &Array[ Index ];
}
template <typename T, typename Allocator> void* operator new( size_t Size, TSparseArray<T, Allocator>& Array, int32 Index )
{
	check( Size == sizeof( T ) );
	Array.InsertUninitialized( Index );
	return &Array[ Index ];
}

struct FScriptSparseArrayLayout
{
	// ElementOffset is at zero offset from the TSparseArrayElementOrFreeListLink - not stored here
	int32 Alignment;
	int32 Size;
};

// Untyped sparse array type for accessing TSparseArray data, like FScriptArray for TArray.
// Must have the same memory representation as a TSet.
template <typename AllocatorType, typename InDerivedType>
class TScriptSparseArray
{
	using DerivedType = std::conditional_t<std::is_void_v<InDerivedType>, TScriptSparseArray, InDerivedType>;

public:
	[[nodiscard]] static constexpr FScriptSparseArrayLayout GetScriptLayout(int32 ElementSize, int32 ElementAlignment)
	{
		FScriptSparseArrayLayout Result;
		Result.Alignment     = FMath::Max(ElementAlignment, (int32)alignof(FFreeListLink));
		Result.Size          = FMath::Max(ElementSize,      (int32)sizeof (FFreeListLink));

		return Result;
	}

	[[nodiscard]] TScriptSparseArray()
		: FirstFreeIndex(-1)
		, NumFreeIndices(0)
	{
	}

	/////////////////////////////////////////////////////
	// Start - intrusive TOptional<TSparseArray> state //
	/////////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TScriptSparseArray;

	[[nodiscard]] explicit TScriptSparseArray(FIntrusiveUnsetOptionalState Tag)
		: Data(Tag)
	{
	}
	[[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState Tag) const
	{
		return Data == Tag;
	}
	///////////////////////////////////////////////////
	// End - intrusive TOptional<TSparseArray> state //
	///////////////////////////////////////////////////

	[[nodiscard]] bool IsValidIndex(int32 Index) const
	{
		return AllocationFlags.IsValidIndex(Index) && AllocationFlags[Index];
	}

	[[nodiscard]] bool IsAllocated(int32 Index) const
	{
		return AllocationFlags[Index];
	}

	[[nodiscard]] bool IsEmpty() const
	{
		return Data.Num() == NumFreeIndices;
	}

	[[nodiscard]] int32 Num() const
	{
		return Data.Num() - NumFreeIndices;
	}

	/** @return The number of elements the array can hold before reallocation. */
	[[nodiscard]] int32 Max() const
	{
		return Data.Max();
	}

	[[nodiscard]] int32 NumUnchecked() const
	{
		return Data.NumUnchecked() - NumFreeIndices;
	}

	/** @return The non-inclusive maximum index of elements in the sparse array. */
	[[nodiscard]] int32 GetMaxIndex() const
	{
		return Data.Num();
	}

	[[nodiscard]] bool IsCompact() const
	{
		return NumFreeIndices == 0;
	}

	[[nodiscard]] void* GetData(int32 Index, const FScriptSparseArrayLayout& Layout)
	{
		return (uint8*)Data.GetData() + Layout.Size * Index;
	}

	[[nodiscard]] const void* GetData(int32 Index, const FScriptSparseArrayLayout& Layout) const
	{
		return (const uint8*)Data.GetData() + Layout.Size * Index;
	}

	void MoveAssign(DerivedType& Other, const FScriptSparseArrayLayout& Layout)
	{
		checkSlow(this != &Other);
		Empty(0, Layout);
		Data.MoveAssign(Other.Data, Layout.Size, Layout.Alignment);
		AllocationFlags.MoveAssign(Other.AllocationFlags);
		FirstFreeIndex = Other.FirstFreeIndex; Other.FirstFreeIndex = 0;
		NumFreeIndices = Other.NumFreeIndices; Other.NumFreeIndices = 0;
	}

	void Empty(int32 Slack, const FScriptSparseArrayLayout& Layout)
	{
		// Free the allocated elements.
		Data.Empty(Slack, Layout.Size, Layout.Alignment);
		FirstFreeIndex = -1;
		NumFreeIndices = 0;
		AllocationFlags.Empty(Slack);
	}

	/**
	 * Adds an uninitialized object to the array.
	 *
	 * @return  The index of the added element.
	 */
	int32 AddUninitialized(const FScriptSparseArrayLayout& Layout)
	{
		int32 Index;
		if (NumFreeIndices)
		{
			// Remove and use the first index from the list of free elements.
			Index = FirstFreeIndex;
			FirstFreeIndex = GetFreeListLink(FirstFreeIndex, Layout)->NextFreeIndex;
			--NumFreeIndices;
			if(NumFreeIndices)
			{
				GetFreeListLink(FirstFreeIndex, Layout)->PrevFreeIndex = -1;
			}
		}
		else
		{
			// Add a new element.
			Index = Data.Add(1, Layout.Size, Layout.Alignment);
			AllocationFlags.Add(false);
		}

		AllocationFlags[Index] = true;

		return Index;
	}

	/** Removes Count elements from the array, starting from Index, without destructing them. */
	void RemoveAtUninitialized(const FScriptSparseArrayLayout& Layout, int32 Index, int32 Count = 1)
	{
		for (; Count; --Count)
		{
			check(AllocationFlags[Index]);

			// Mark the element as free and add it to the free element list.
			if(NumFreeIndices)
			{
				GetFreeListLink(FirstFreeIndex, Layout)->PrevFreeIndex = Index;
			}

			auto* IndexData = GetFreeListLink(Index, Layout);
			IndexData->PrevFreeIndex = -1;
			IndexData->NextFreeIndex = NumFreeIndices > 0 ? FirstFreeIndex : INDEX_NONE;
			FirstFreeIndex = Index;
			++NumFreeIndices;
			AllocationFlags[Index] = false;

			++Index;
		}
	}

private:
	TScriptArray   <typename AllocatorType::ElementAllocator>  Data;
	TScriptBitArray<typename AllocatorType::BitArrayAllocator> AllocationFlags;
	int32                                                      FirstFreeIndex = -1;
	int32                                                      NumFreeIndices = 0;

	// This function isn't intended to be called, just to be compiled to validate the correctness of the type.
	static void CheckConstraints()
	{
		typedef TScriptSparseArray  ScriptType;
		typedef TSparseArray<int32> RealType;

		// Check that the class footprint is the same
		static_assert(sizeof (ScriptType) == sizeof (RealType), "TScriptSparseArray's size doesn't match TSparseArray");
		static_assert(alignof(ScriptType) == alignof(RealType), "TScriptSparseArray's alignment doesn't match TSparseArray");

		// Check member sizes
		static_assert(sizeof(DeclVal<ScriptType>().Data)            == sizeof(DeclVal<RealType>().Data),            "TScriptSparseArray's Data member size does not match TSparseArray's");
		static_assert(sizeof(DeclVal<ScriptType>().AllocationFlags) == sizeof(DeclVal<RealType>().AllocationFlags), "TScriptSparseArray's AllocationFlags member size does not match TSparseArray's");
		static_assert(sizeof(DeclVal<ScriptType>().FirstFreeIndex)  == sizeof(DeclVal<RealType>().FirstFreeIndex),  "TScriptSparseArray's FirstFreeIndex member size does not match TSparseArray's");
		static_assert(sizeof(DeclVal<ScriptType>().NumFreeIndices)  == sizeof(DeclVal<RealType>().NumFreeIndices),  "TScriptSparseArray's NumFreeIndices member size does not match TSparseArray's");

		// Check member offsets
		static_assert(STRUCT_OFFSET(ScriptType, Data)            == STRUCT_OFFSET(RealType, Data),            "TScriptSparseArray's Data member offset does not match TSparseArray's");
		static_assert(STRUCT_OFFSET(ScriptType, AllocationFlags) == STRUCT_OFFSET(RealType, AllocationFlags), "TScriptSparseArray's AllocationFlags member offset does not match TSparseArray's");
		static_assert(STRUCT_OFFSET(ScriptType, FirstFreeIndex)  == STRUCT_OFFSET(RealType, FirstFreeIndex),  "TScriptSparseArray's FirstFreeIndex member offset does not match TSparseArray's");
		static_assert(STRUCT_OFFSET(ScriptType, NumFreeIndices)  == STRUCT_OFFSET(RealType, NumFreeIndices),  "TScriptSparseArray's NumFreeIndices member offset does not match TSparseArray's");

		// Check free index offsets
		static_assert(STRUCT_OFFSET(ScriptType::FFreeListLink, PrevFreeIndex) == STRUCT_OFFSET(RealType::FElementOrFreeListLink, PrevFreeIndex), "TScriptSparseArray's FFreeListLink's PrevFreeIndex member offset does not match TSparseArray's");
		static_assert(STRUCT_OFFSET(ScriptType::FFreeListLink, NextFreeIndex) == STRUCT_OFFSET(RealType::FElementOrFreeListLink, NextFreeIndex), "TScriptSparseArray's FFreeListLink's NextFreeIndex member offset does not match TSparseArray's");
	}

	struct FFreeListLink
	{
		/** If the element isn't allocated, this is a link to the previous element in the array's free list. */
		int32 PrevFreeIndex;

		/** If the element isn't allocated, this is a link to the next element in the array's free list. */
		int32 NextFreeIndex;
	};

	/** Accessor for the element or free list data. */
	UE_FORCEINLINE_HINT FFreeListLink* GetFreeListLink(int32 Index, const FScriptSparseArrayLayout& Layout)
	{
		return (FFreeListLink*)GetData(Index, Layout);
	}

public:
	// These should really be private, because they shouldn't be called, but there's a bunch of code
	// that needs to be fixed first.
	TScriptSparseArray(const TScriptSparseArray&) { check(false); }
	void operator=(const TScriptSparseArray&) { check(false); }
};

template <typename AllocatorType, typename InDerivedType>
struct TIsZeroConstructType<TScriptSparseArray<AllocatorType, InDerivedType>>
{
	enum { Value = true };
};

class FScriptSparseArray : public TScriptSparseArray<FDefaultSparseArrayAllocator, FScriptSparseArray>
{
	using Super = TScriptSparseArray<FDefaultSparseArrayAllocator, FScriptSparseArray>;

public:
	using Super::Super;

	/////////////////////////////////////////////////////
	// Start - intrusive TOptional<FSparseArray> state //
	/////////////////////////////////////////////////////
	using IntrusiveUnsetOptionalStateType = FScriptSparseArray;
	///////////////////////////////////////////////////
	// End - intrusive TOptional<FSparseArray> state //
	///////////////////////////////////////////////////
};

/**
 * A placement new operator which constructs an element in a sparse array allocation.
 */
inline void* operator new(size_t Size,const FSparseArrayAllocationInfo& Allocation)
{
	UE_ASSUME(Allocation.Pointer);
	return Allocation.Pointer;
}
inline void operator delete(void* ,const FSparseArrayAllocationInfo& Allocation)
{
}

/** Serializer. */
template<typename ElementType,typename Allocator>
FArchive& operator<<(FArchive& Ar,TSparseArray<ElementType, Allocator>& Array)
{
	Array.CountBytes(Ar);
	if( Ar.IsLoading() )
	{
		// Load array.
		int32 NewNumElements = 0;
		Ar << NewNumElements;
		Array.Empty( NewNumElements );
		for(int32 ElementIndex = 0;ElementIndex < NewNumElements;ElementIndex++)
		{
			Ar << *::new(Array.AddUninitialized())ElementType;
		}
	}
	else
	{
		// Save array.
		int32 NewNumElements = Array.Num();
		Ar << NewNumElements;
		for(typename TSparseArray<ElementType, Allocator>::TIterator It(Array);It;++It)
		{
			Ar << *It;
		}
	}
	return Ar;
}

/** Structured archive serializer. */
template<typename ElementType,typename Allocator>
void operator<<(FStructuredArchive::FSlot Slot, TSparseArray<ElementType, Allocator>& InArray)
{
	int32 NumElements = InArray.Num();
	FStructuredArchive::FArray Array = Slot.EnterArray(NumElements);
	if (Slot.GetUnderlyingArchive().IsLoading())
	{
		InArray.Empty(NumElements);

		for (int32 Index = 0; Index < NumElements; ++Index)
		{
			if (Slot.GetUnderlyingArchive().IsCriticalError())
			{
				return;
			}

			FStructuredArchive::FSlot ElementSlot = Array.EnterElement();
			ElementSlot << *::new(InArray.AddUninitialized())ElementType;
		}
	}
	else
	{
		for (typename TSparseArray<ElementType, Allocator>::TIterator It(InArray); It; ++It)
		{
			FStructuredArchive::FSlot ElementSlot = Array.EnterElement();
			ElementSlot << *It;
		}
	}
}
