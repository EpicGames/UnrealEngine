// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef UE_TSPARSE_SET
#error "SparseSet.h.inl should only be included after defining UE_TSPARSE_SET"
#endif

#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/ContainerElementTypeCompatibility.h"
#include "Containers/SetUtilities.h"
#include "Containers/SparseArray.h"
#include "Containers/SparseSetElement.h"
#include "ContainersFwd.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/StructBuilder.h"
#include "Serialization/MemoryImageWriter.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/Function.h"
#include "Templates/RetainedRef.h"
#include "Templates/Sorting.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Traits/IsTriviallyRelocatable.h"
#include <initializer_list>
#include <type_traits>

template <typename AllocatorType>
class TScriptSparseSet;

#define TSETPRIVATEFRIEND UE_JOIN(UE_TSPARSE_SET, PrivateFriend)

#if !defined(UE_USE_SPARSE_SET_STRICT_MODE)
#define UE_USE_SPARSE_SET_STRICT_MODE 0
#endif

#define DO_SPARSE_SET_ITERATOR_CHECKS DO_CHECK && UE_USE_SPARSE_SET_STRICT_MODE

/**
 * A set with an optional KeyFuncs parameters for customizing how the elements are compared and searched.
 * E.g. You can specify a mapping from elements to keys if you want to find elements by specifying a subset of
 * the element type.  It uses a TSparseArray of the elements, and also links the elements into a hash with a
 * number of buckets proportional to the number of elements.  Addition, removal, and finding are O(1).
 *
 * The ByHash() functions are somewhat dangerous but particularly useful in two scenarios:
 * -- Heterogeneous lookup to avoid creating expensive keys like FString when looking up by const TCHAR*.
 *	  You must ensure the hash is calculated in the same way as ElementType is hashed.
 *    If possible put both ComparableKey and ElementType hash functions next to each other in the same header
 *    to avoid bugs when the ElementType hash function is changed.
 * -- Reducing contention around hash tables protected by a lock. It is often important to incur
 *    the cache misses of reading key data and doing the hashing *before* acquiring the lock.
 *
 **/
template<
	typename InElementType,
	typename KeyFuncs /*= DefaultKeyFuncs<ElementType>*/,
	typename Allocator /*= FDefaultSparseSetAllocator*/
	>
class UE_TSPARSE_SET
{
public:
	typedef InElementType ElementType;
	typedef KeyFuncs    KeyFuncsType;
	typedef Allocator   AllocatorType;

	using SizeType = typename Allocator::SparseArrayAllocator::ElementAllocator::SizeType;

	static_assert(std::is_same_v<SizeType, int32>, "UE_TSPARSE_SET currently only supports 32-bit allocators");

private:
	using USizeType = std::make_unsigned_t<SizeType>;

	template <typename>
	friend class TScriptSparseSet;

	typedef typename KeyFuncs::KeyInitType     KeyInitType;
	typedef typename KeyFuncs::ElementInitType ElementInitType;

	typedef TSparseSetElement<InElementType> SetElementType;

public:
	/** Initialization constructor. */
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr UE_TSPARSE_SET() = default;

	[[nodiscard]] explicit consteval UE_TSPARSE_SET(EConstEval)
	: Elements(ConstEval)
	, Hash(ConstEval)
	{
	}

	/** Copy constructor. */
	[[nodiscard]] UE_FORCEINLINE_HINT UE_TSPARSE_SET(const UE_TSPARSE_SET& Copy)
	{
		*this = Copy;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT explicit UE_TSPARSE_SET(TArrayView<const ElementType> InArrayView)
	{
		Append(InArrayView);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT explicit UE_TSPARSE_SET(TArray<ElementType>&& InArray)
	{
		Append(MoveTemp(InArray));
	}

	/** Destructor. */
	UE_FORCEINLINE_HINT ~UE_TSPARSE_SET()
	{
		UE_STATIC_ASSERT_WARN(TIsTriviallyRelocatable_V<InElementType>, "TSet can only be used with trivially relocatable types");
		HashSize = 0;
	}

	///////////////////////////////////////////////////
	// Start - intrusive TOptional<UE_TSPARSE_SET> state //
	///////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = UE_TSPARSE_SET;

	[[nodiscard]] explicit UE_TSPARSE_SET(FIntrusiveUnsetOptionalState Tag)
		: Elements(Tag)
	{
	}
	[[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState Tag) const
	{
		return Elements == Tag;
	}
	/////////////////////////////////////////////////
	// End - intrusive TOptional<UE_TSPARSE_SET> state //
	/////////////////////////////////////////////////

	/** Assignment operator. */
	UE_TSPARSE_SET& operator=(const UE_TSPARSE_SET& Copy)
	{
		if (this != &Copy)
		{
			UE::Core::Private::CopyHash(Hash, HashSize, Copy.Hash, Copy.HashSize);

			Elements = Copy.Elements;
		}
		return *this;
	}

private:
	template <typename SetType>
	static inline void Move(SetType& ToSet, SetType& FromSet)
	{
		ToSet.Elements = (ElementArrayType&&)FromSet.Elements;

		ToSet.Hash.MoveToEmpty(FromSet.Hash);

		ToSet  .HashSize = FromSet.HashSize;
		FromSet.HashSize = 0;
	}

public:
	/** Initializer list constructor. */
	[[nodiscard]] UE_TSPARSE_SET(std::initializer_list<ElementType> InitList)
		: HashSize(0)
	{
		Append(InitList);
	}

	/** Move constructor. */
	[[nodiscard]] UE_TSPARSE_SET(UE_TSPARSE_SET&& Other)
		: HashSize(0)
	{
		this->Move(*this, Other);
	}

	/** Move assignment operator. */
	UE_TSPARSE_SET& operator=(UE_TSPARSE_SET&& Other)
	{
		if (this != &Other)
		{
			this->Move(*this, Other);
		}

		return *this;
	}

	/** Constructor for moving elements from a UE_TSPARSE_SET with a different SetAllocator */
	template<typename OtherAllocator>
	[[nodiscard]] UE_TSPARSE_SET(UE_TSPARSE_SET<ElementType, KeyFuncs, OtherAllocator>&& Other)
		: HashSize(0)
	{
		Append(MoveTemp(Other));
	}

	/** Constructor for copying elements from a UE_TSPARSE_SET with a different SetAllocator */
	template<typename OtherAllocator>
	[[nodiscard]] UE_TSPARSE_SET(const UE_TSPARSE_SET<ElementType, KeyFuncs, OtherAllocator>& Other)
		: HashSize(0)
	{
		Append(Other);
	}

	/** Assignment operator for moving elements from a UE_TSPARSE_SET with a different SetAllocator */
	template<typename OtherAllocator>
	UE_TSPARSE_SET& operator=(UE_TSPARSE_SET<ElementType, KeyFuncs, OtherAllocator>&& Other)
	{
		Reset();
		Append(MoveTemp(Other));
		return *this;
	}

	/** Assignment operator for copying elements from a UE_TSPARSE_SET with a different SetAllocator */
	template<typename OtherAllocator>
	UE_TSPARSE_SET& operator=(const UE_TSPARSE_SET<ElementType, KeyFuncs, OtherAllocator>& Other)
	{
		Reset();
		Append(Other);
		return *this;
	}

	/** Initializer list assignment operator */
	UE_TSPARSE_SET& operator=(std::initializer_list<ElementType> InitList)
	{
		Reset();
		Append(InitList);
		return *this;
	}

	/**
	 * Removes all elements from the set, potentially leaving space allocated for an expected number of elements about to be added.
	 * @param ExpectedNumElements - The number of elements about to be added to the set.
	 */
	void Empty(int32 ExpectedNumElements = 0)
	{
		// Empty the elements array, and reallocate it for the expected number of elements.
		const int32 DesiredHashSize = Allocator::GetNumberOfHashBuckets(ExpectedNumElements);
		const bool ShouldDoRehash = ShouldRehash(ExpectedNumElements, DesiredHashSize, EAllowShrinking::Yes);

		if (!ShouldDoRehash)
		{
			// If the hash was already the desired size, clear the references to the elements that have now been removed.
			UnhashElements();
		}

		Elements.Empty(ExpectedNumElements);

		// Resize the hash to the desired size for the expected number of elements.
		if (ShouldDoRehash)
		{
			HashSize = DesiredHashSize;
			Rehash();
		}
	}

	/** Efficiently empties out the set but preserves all allocations and capacities */
	void Reset()
	{
		if (Num() == 0)
		{
			return;
		}

		// Reset the elements array.
		UnhashElements();
		Elements.Reset();
	}

	/** Shrinks the set's element storage to avoid slack. */
	inline void Shrink()
	{
		Elements.Shrink();
		Relax();
	}

	/** Compacts the allocated elements into a contiguous range. */
	inline void Compact()
	{
		if (Elements.Compact())
		{
			HashSize = Allocator::GetNumberOfHashBuckets(Elements.Num());
			Rehash();
		}
	}

	/** Compacts the allocated elements into a contiguous range. Does not change the iteration order of the elements. */
	inline void CompactStable()
	{
		if (Elements.CompactStable())
		{
			HashSize = Allocator::GetNumberOfHashBuckets(Elements.Num());
			Rehash();
		}
	}

	/** Preallocates enough memory to contain Number elements */
	inline void Reserve(int32 Number)
	{
		// makes sense only when Number > Elements.Num() since TSparseArray::Reserve
		// does any work only if that's the case
		if ((USizeType)Number > (USizeType)Elements.Num())
		{
			// Trap negative reserves
			if (Number < 0)
			{
				UE::Core::Private::OnInvalidSetNum((unsigned long long)Number);
			}

			// Preallocates memory for array of elements
			Elements.Reserve(Number);

			// Calculate the corresponding hash size for the specified number of elements.
			const int32 NewHashSize = Allocator::GetNumberOfHashBuckets(Number);

			// If the hash hasn't been created yet, or is smaller than the corresponding hash size, rehash
			// to force a preallocation of the hash table
			if(!HashSize || HashSize < NewHashSize)
			{
				HashSize = NewHashSize;
				Rehash();
			}
		}
	}

	/** Relaxes the set's hash to a size strictly bounded by the number of elements in the set. */
	UE_FORCEINLINE_HINT void Relax()
	{
		ConditionalRehash(Elements.Num(), EAllowShrinking::Yes);
	}

	/** 
	 * Helper function to return the amount of memory allocated by this container
	 * Only returns the size of allocations made directly by the container, not the elements themselves.
	 * @return number of bytes allocated by this container
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT SIZE_T GetAllocatedSize( void ) const
	{
		return Elements.GetAllocatedSize() + Hash.GetAllocatedSize(HashSize, sizeof(FSetElementId));
	}

	/** Tracks the container's memory use through an archive. */
	inline void CountBytes(FArchive& Ar) const
	{
		Elements.CountBytes(Ar);
		Ar.CountBytes(HashSize * sizeof(int32),HashSize * sizeof(FSetElementId));
	}

	/**
	 * Returns true if the sets is empty and contains no elements.
	 *
	 * @returns True if the set is empty.
	 * @see Num
	 */
	[[nodiscard]] bool IsEmpty() const
	{
		return Elements.IsEmpty();
	}

	/** @return the number of elements. */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Num() const
	{
		return Elements.Num();
	}

	/** @return The number of elements the set can hold before reallocation. */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Max() const
	{
		return Elements.Max();
	}

	/** @return The non-inclusive maximum index of elements in the set. */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 GetMaxIndex() const
	{
		return Elements.GetMaxIndex();
	}

	/**
	 * Checks whether an element id is valid.
	 * @param Id - The element id to check.
	 * @return true if the element identifier refers to a valid element in this set.
	 */
	[[nodiscard]] inline bool IsValidId(FSetElementId Id) const
	{
		SizeType Index = Id.AsInteger();
		return Index != INDEX_NONE &&
			Index >= 0 &&
			Index < Elements.GetMaxIndex() &&
			Elements.IsAllocated(Index);
	}

	/** Accesses the identified element's value. Element must be valid (see @IsValidId). */
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& operator[](FSetElementId Id)
	{
		return Elements[Id.AsInteger()].Value;
	}

	/** Accesses the identified element's value. Element must be valid (see @IsValidId). */
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType& operator[](FSetElementId Id) const
	{
		return Elements[Id.AsInteger()].Value;
	}

	/** Accesses the identified element's value. Element must be valid (see @IsValidId). */
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& Get(FSetElementId Id)
	{
		return Elements[Id.AsInteger()].Value;
	}

	/** Accesses the identified element's value. Element must be valid (see @IsValidId). */
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType& Get(FSetElementId Id) const
	{
		return Elements[Id.AsInteger()].Value;
	}

	/**
	 * Adds an element to the set.
	 *
	 * @param	InElement					Element to add to set
	 * @param	bIsAlreadyInSetPtr	[out]	Optional pointer to bool that will be set depending on whether element is already in set
	 * @return	A pointer to the element stored in the set.
	 */
	UE_FORCEINLINE_HINT FSetElementId Add(const InElementType&  InElement, bool* bIsAlreadyInSetPtr = nullptr)
	{
		return Emplace(InElement, bIsAlreadyInSetPtr);
	}
	UE_FORCEINLINE_HINT FSetElementId Add(InElementType&& InElement, bool* bIsAlreadyInSetPtr = nullptr)
	{
		return Emplace(MoveTempIfPossible(InElement), bIsAlreadyInSetPtr);
	}

	/**
	 * Adds an element to the set if not already present and returns a reference to the added or existing element.
	 *
	 * @param	InElement					Element to add to set
	 * @param	bIsAlreadyInSetPtr	[out]	Optional pointer to bool that will be set depending on whether element is already in set
	 * @return	A reference to the element stored in the set.
	 */
	UE_FORCEINLINE_HINT ElementType& FindOrAdd(const InElementType& InElement, bool* bIsAlreadyInSetPtr = nullptr)
	{
		return FindOrAddByHash(KeyFuncs::GetKeyHash(KeyFuncs::GetSetKey(InElement)), InElement, bIsAlreadyInSetPtr);
	}
	UE_FORCEINLINE_HINT ElementType& FindOrAdd(InElementType&& InElement, bool* bIsAlreadyInSetPtr = nullptr)
	{
		const uint32 KeyHash = KeyFuncs::GetKeyHash(KeyFuncs::GetSetKey(InElement));
		return FindOrAddByHash(KeyHash, MoveTempIfPossible(InElement), bIsAlreadyInSetPtr);
	}

	/**
	 * Adds an element to the set.
	 *
	 * @see		Class documentation section on ByHash() functions
	 * @param	KeyHash						A precomputed hash value, calculated in the same way as ElementType is hashed.
	 * @param	InElement					Element to add to set
	 * @param	bIsAlreadyInSetPtr	[out]	Optional pointer to bool that will be set depending on whether element is already in set
     * @return  A handle to the element stored in the set
     */
	UE_FORCEINLINE_HINT FSetElementId AddByHash(uint32 KeyHash, const InElementType& InElement, bool* bIsAlreadyInSetPtr = nullptr)
	{
		return EmplaceByHash(KeyHash, InElement, bIsAlreadyInSetPtr);
	}
	UE_FORCEINLINE_HINT FSetElementId AddByHash(uint32 KeyHash,		 InElementType&& InElement, bool* bIsAlreadyInSetPtr = nullptr)
	{
		return EmplaceByHash(KeyHash, MoveTempIfPossible(InElement), bIsAlreadyInSetPtr);
	}

	/**
	 * Adds an element to the set if not already present and returns a reference to the added or existing element.
	 *
	 * @see		Class documentation section on ByHash() functions
	 * @param	KeyHash						A precomputed hash value, calculated in the same way as ElementType is hashed.
	 * @param	InElement					Element to add to set
	 * @param	bIsAlreadyInSetPtr	[out]	Optional pointer to bool that will be set depending on whether element is already in set
	 * @return  A reference to the element stored in the set
	 */
	template <typename ElementReferenceType>
	ElementType& FindOrAddByHash(uint32 KeyHash, ElementReferenceType&& InElement, bool* bIsAlreadyInSetPtr = nullptr)
	{
		SizeType ExistingIndex = FindIndexByHash(KeyHash, KeyFuncs::GetSetKey(InElement));
		bool bIsAlreadyInSet = ExistingIndex != INDEX_NONE;
		if (bIsAlreadyInSetPtr)
		{
			*bIsAlreadyInSetPtr = bIsAlreadyInSet;
		}
		if (bIsAlreadyInSet)
		{
			return Elements[ExistingIndex].Value;
		}

		// Create a new element.
		FSparseArrayAllocationInfo ElementAllocation = Elements.AddUninitialized();
		SetElementType& Element = *new (ElementAllocation) SetElementType(Forward<ElementReferenceType>(InElement));
		RehashOrLink(KeyHash, Element, ElementAllocation.Index);
		return Element.Value;
	}

private:
	bool TryReplaceExisting(uint32 KeyHash, SetElementType& Element, SizeType& InOutElementIndex, bool* bIsAlreadyInSetPtr)
	{
		bool bIsAlreadyInSet = false;
		if constexpr (!KeyFuncs::bAllowDuplicateKeys)
		{
			// If the set doesn't allow duplicate keys, check for an existing element with the same key as the element being added.

			// Don't bother searching for a duplicate if this is the first element we're adding
			if (Elements.Num() != 1)
			{
				SizeType ExistingIndex = FindIndexByHash(KeyHash, KeyFuncs::GetSetKey(Element.Value));
				bIsAlreadyInSet = ExistingIndex != INDEX_NONE;
				if (bIsAlreadyInSet)
				{
					// If there's an existing element with the same key as the new element, replace the existing element with the new element.
					MoveByRelocate(Elements[ExistingIndex].Value, Element.Value);

					// Then remove the new element.
					Elements.RemoveAtUninitialized(InOutElementIndex);

					// Then point the return value at the replaced element.
					InOutElementIndex = ExistingIndex;
				}
			}
		}
		if (bIsAlreadyInSetPtr)
		{
			*bIsAlreadyInSetPtr = bIsAlreadyInSet;
		}
		return bIsAlreadyInSet;
	}

	inline void RehashOrLink(uint32 KeyHash, SetElementType& Element, SizeType ElementIndex)
	{
		// Check if the hash needs to be resized.
		if (!ConditionalRehash(Elements.Num(), EAllowShrinking::No))
		{
			// If the rehash didn't add the new element to the hash, add it.
			LinkElement(ElementIndex, Element, KeyHash);
		}
	}

public:
	/**
	 * Adds an element to the set.
	 *
	 * @param	Arg							The argument(s) to be forwarded to the set element's constructor.
	 * @param	bIsAlreadyInSetPtr	[out]	Optional pointer to bool that will be set depending on whether element is already in set
	 * @return	A handle to the element stored in the set.
	 */
	template <typename ArgType = ElementType>
	FSetElementId Emplace(ArgType&& Arg, bool* bIsAlreadyInSetPtr = nullptr)
	{
		// Create a new element.
		FSparseArrayAllocationInfo ElementAllocation = Elements.AddUninitialized();
		SetElementType& Element = *new (ElementAllocation) SetElementType(Forward<ArgType>(Arg));

		SizeType NewHashIndex = ElementAllocation.Index;

		uint32 KeyHash = KeyFuncs::GetKeyHash(KeyFuncs::GetSetKey(Element.Value));
		if (!TryReplaceExisting(KeyHash, Element, NewHashIndex, bIsAlreadyInSetPtr))
		{
			RehashOrLink(KeyHash, Element, NewHashIndex);
		}
		return FSetElementId::FromInteger(NewHashIndex);
	}

    /**
     * Adds an element to the set by constructing the ElementType in-place with multiple args.
     *
     * @param   EInPlace   Tag to disambiguate in-place construction.
     * @param   InArgs     Arguments forwarded to ElementType's constructor.
     * @return  Pair of (element id, whether an equivalent element already existed).
     */
    template <typename... ArgTypes>
    TPair<FSetElementId, bool> Emplace(EInPlace, ArgTypes&&... InArgs)
    {
		// Create a new element.
        FSparseArrayAllocationInfo ElementAllocation = Elements.AddUninitialized();
		SetElementType& Element = *new (ElementAllocation) SetElementType(Forward<ArgTypes>(InArgs)...);

        SizeType NewHashIndex = ElementAllocation.Index;
        const uint32 KeyHash = KeyFuncs::GetKeyHash(KeyFuncs::GetSetKey(Element.Value));

        bool bAlreadyInSet = false;
        if (!TryReplaceExisting(KeyHash, Element, NewHashIndex, &bAlreadyInSet))
        {
            RehashOrLink(KeyHash, Element, NewHashIndex);
        }

        return TPair<FSetElementId, bool>(FSetElementId::FromInteger(NewHashIndex), bAlreadyInSet);
    }
	
	/**
	 * Adds an element to the set.
	 *
	 * @see		Class documentation section on ByHash() functions
	 * @param	KeyHash                     A precomputed hash value, calculated in the same way as ElementType is hashed.
	 * @param	Arg							The argument(s) to be forwarded to the set element's constructor.
	 * @param	bIsAlreadyInSetPtr	[out]	Optional pointer to bool that will be set depending on whether element is already in set
	 * @return	A handle to the element stored in the set.
	 */
	template <typename ArgType = ElementType>
	FSetElementId EmplaceByHash(uint32 KeyHash, ArgType&& Args, bool* bIsAlreadyInSetPtr = nullptr)
	{
		// Create a new element.
		FSparseArrayAllocationInfo ElementAllocation = Elements.AddUninitialized();
		SetElementType& Element = *new (ElementAllocation) SetElementType(Forward<ArgType>(Args));

		SizeType NewHashIndex = ElementAllocation.Index;

		if (!TryReplaceExisting(KeyHash, Element, NewHashIndex, bIsAlreadyInSetPtr))
		{
			RehashOrLink(KeyHash, Element, NewHashIndex);
		}
		return FSetElementId::FromInteger(NewHashIndex);
	}

    /**
     * Adds an element to the set by constructing in-place with multiple args, using a precomputed hash.
	 *
     * @see     Class documentation section on ByHash() functions.
	 * @param	KeyHash    A precomputed hash value, calculated in the same way as ElementType is hashed.
     * @param   EInPlace   Tag to disambiguate in-place construction.
     * @param   InArgs     Arguments forwarded to ElementType's constructor.
     * @return  Pair of (element id, whether an equivalent element already existed).
     */
    template <typename... ArgTypes>
    TPair<FSetElementId, bool> EmplaceByHash(EInPlace, uint32 KeyHash, ArgTypes&&... InArgs)
    {
		// Create a new element.
        FSparseArrayAllocationInfo ElementAllocation = Elements.AddUninitialized();
		SetElementType& Element = *new (ElementAllocation) SetElementType(Forward<ArgTypes>(InArgs)...);

        SizeType NewHashIndex = ElementAllocation.Index;
        bool bAlreadyInSet = false;
        if (!TryReplaceExisting(KeyHash, Element, NewHashIndex, &bAlreadyInSet))
        {
            RehashOrLink(KeyHash, Element, NewHashIndex);
        }
        return TPair<FSetElementId, bool>(FSetElementId::FromInteger(NewHashIndex), bAlreadyInSet);
    }

	void Append(TArrayView<const ElementType> InElements)
	{
		Reserve(Elements.Num() + InElements.Num());
		for (const ElementType& Element : InElements)
		{
			Add(Element);
		}
	}

	template<typename ArrayAllocator>
	void Append(TArray<ElementType, ArrayAllocator>&& InElements)
	{
		Reserve(Elements.Num() + InElements.Num());
		for (ElementType& Element : InElements)
		{
			Add(MoveTempIfPossible(Element));
		}
		InElements.Reset();
	}

	/**
	 * Add all items from another set to our set (union without creating a new set)
	 * @param OtherSet - The other set of items to add.
	 */
	template<typename OtherAllocator>
	void Append(const UE_TSPARSE_SET<ElementType, KeyFuncs, OtherAllocator>& OtherSet)
	{
		Reserve(Elements.Num() + OtherSet.Num());
		for (const ElementType& Element : OtherSet)
		{
			Add(Element);
		}
	}

	template<typename OtherAllocator>
	void Append(UE_TSPARSE_SET<ElementType, KeyFuncs, OtherAllocator>&& OtherSet)
	{
		Reserve(Elements.Num() + OtherSet.Num());
		for (ElementType& Element : OtherSet)
		{
			Add(MoveTempIfPossible(Element));
		}
		OtherSet.Reset();
	}

	void Append(std::initializer_list<ElementType> InitList)
	{
		Reserve(Elements.Num() + (int32)InitList.size());
		for (const ElementType& Element : InitList)
		{
			Add(Element);
		}
	}

private:
	void RemoveByIndex(SizeType ElementIndex)
	{
		checkf(Elements.IsValidIndex(ElementIndex), TEXT("Invalid ElementIndex passed to UE_TSPARSE_SET::RemoveByIndex"));

		const SetElementType& ElementBeingRemoved = Elements[ElementIndex];

		// Remove the element from the hash.
		FSetElementId* HashPtr              = Hash.GetAllocation();
		FSetElementId* NextElementIndexIter = &HashPtr[ElementBeingRemoved.HashIndex];
		for (;;)
		{
			SizeType NextElementIndex = NextElementIndexIter->AsInteger();
			checkf(NextElementIndex != INDEX_NONE, TEXT("Corrupt hash"));

			if (NextElementIndex == ElementIndex)
			{
				*NextElementIndexIter = ElementBeingRemoved.HashNextId;
				break;
			}

			NextElementIndexIter = &Elements[NextElementIndex].HashNextId;
		}

		// Remove the element from the elements array.
		Elements.RemoveAt(ElementIndex);
	}

public:
	/**
	 * Removes an element from the set.
	 * @param Element - A pointer to the element in the set, as returned by Add or Find.
	 */
	void Remove(FSetElementId ElementId)
	{
		RemoveByIndex(ElementId.AsInteger());
	}

	/**
	 * Removes an element from the set while maintaining set order.
	 * @param Element - A pointer to the element in the set, as returned by Add or Find.
	 */
	void RemoveStable(FSetElementId ElementId)
	{
		Remove(ElementId);
		CompactStable();
	}

private:
	/**
	 * Finds an element with a pre-calculated hash and a key that can be compared to KeyType
	 * @see	Class documentation section on ByHash() functions
	 * @return The element id that matches the key and hash or an invalid element id
	 */
	template <typename ComparableKey>
	[[nodiscard]] SizeType FindIndexByHash(uint32 KeyHash, const ComparableKey& Key) const
	{
		if (Elements.Num() == 0)
		{
			return INDEX_NONE;
		}

		FSetElementId* HashPtr      = Hash.GetAllocation();
		SizeType       ElementIndex = HashPtr[KeyHash & (HashSize - 1)].AsInteger();
		for (;;)
		{
			if (ElementIndex == INDEX_NONE)
			{
				return INDEX_NONE;
			}

			if (KeyFuncs::Matches(KeyFuncs::GetSetKey(Elements[ElementIndex].Value), Key))
			{
				// Return the first match, regardless of whether the set has multiple matches for the key or not.
				return ElementIndex;
			}

			ElementIndex = Elements[ElementIndex].HashNextId.AsInteger();
		}
	}

public:
	/**
	 * Finds any element in the set and returns a pointer to it.
	 * Callers should not depend on particular patterns in the behaviour of this function.
	 * @return A pointer to an arbitrary element, or nullptr if the container is empty.
	 */
	[[nodiscard]] ElementType* FindArbitraryElement()
	{
		// The goal of this function is to be fast, and so the implementation may be improved at any time even if it gives different results.

		int32 Result = Elements.FindArbitraryElementIndex();
		return (Result != INDEX_NONE) ? &Elements[Result].Value : nullptr;
	}
	[[nodiscard]] const ElementType* FindArbitraryElement() const
	{
		return const_cast<UE_TSPARSE_SET*>(this)->FindArbitraryElement();
	}

	/**
	 * Finds an element with the given key in the set.
	 * @param Key - The key to search for.
	 * @return The id of the set element matching the given key, or the NULL id if none matches.
	 */
	[[nodiscard]] FSetElementId FindId(KeyInitType Key) const
	{
		return FSetElementId::FromInteger(FindIndexByHash(KeyFuncs::GetKeyHash(Key), Key));
	}

	/**
	 * Finds an element with a pre-calculated hash and a key that can be compared to KeyType
	 * @see	Class documentation section on ByHash() functions
	 * @return The element id that matches the key and hash or an invalid element id
	 */
	template<typename ComparableKey>
	[[nodiscard]] FSetElementId FindIdByHash(uint32 KeyHash, const ComparableKey& Key) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Disable deprecations warnings to stop warnings being thrown by our check macro.
		checkSlow(KeyHash == KeyFuncs::GetKeyHash(Key));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		return FSetElementId::FromInteger(FindIndexByHash(KeyHash, Key));
	}

	/**
	 * Finds an element with the given key in the set.
	 * @param Key - The key to search for.
	 * @return A pointer to an element with the given key.  If no element in the set has the given key, this will return NULL.
	 */
	[[nodiscard]] inline ElementType* Find(KeyInitType Key)
	{
		SizeType ElementIndex = FindIndexByHash(KeyFuncs::GetKeyHash(Key), Key);
		if (ElementIndex != INDEX_NONE)
		{
			return &Elements[ElementIndex].Value;
		}
		else
		{
			return nullptr;
		}
	}

	/**
	 * Finds an element with the given key in the set.
	 * @param Key - The key to search for.
	 * @return A const pointer to an element with the given key.  If no element in the set has the given key, this will return NULL.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType* Find(KeyInitType Key) const
	{
		return const_cast<UE_TSPARSE_SET*>(this)->Find(Key);
	}

	/**
	 * Finds an element with a pre-calculated hash and a key that can be compared to KeyType.
	 * @see	Class documentation section on ByHash() functions
	 * @return A pointer to the contained element or nullptr.
	 */
	template<typename ComparableKey>
	[[nodiscard]] ElementType* FindByHash(uint32 KeyHash, const ComparableKey& Key)
	{
		SizeType ElementIndex = FindIndexByHash(KeyHash, Key);
		if (ElementIndex != INDEX_NONE)
		{
			return &Elements[ElementIndex].Value;
		}
		else
		{
			return nullptr;
		}
	}

	template<typename ComparableKey>
	[[nodiscard]] const ElementType* FindByHash(uint32 KeyHash, const ComparableKey& Key) const
	{
		return const_cast<UE_TSPARSE_SET*>(this)->FindByHash(KeyHash, Key);
	}

private:
	template<typename ComparableKey>
	inline int32 RemoveImpl(uint32 KeyHash, const ComparableKey& Key)
	{
		int32 NumRemovedElements = 0;

		FSetElementId* NextElementId = &GetTypedHash(KeyHash);
		while (NextElementId->IsValidId())
		{
			const int32 ElementIndex = NextElementId->AsInteger();
			SetElementType& Element = Elements[ElementIndex];

			if (KeyFuncs::Matches(KeyFuncs::GetSetKey(Element.Value), Key))
			{
				// This element matches the key, remove it from the set.  Note that Remove sets *NextElementId to point to the next
				// element after the removed element in the hash bucket.
				RemoveByIndex(ElementIndex);
				NumRemovedElements++;

				if constexpr (!KeyFuncs::bAllowDuplicateKeys)
				{
					// If the hash disallows duplicate keys, we're done removing after the first matched key.
					break;
				}
			}
			else
			{
				NextElementId = &Element.HashNextId;
			}
		}

		return NumRemovedElements;
	}

public:
	/**
	 * Removes all elements from the set matching the specified key.
	 * @param Key - The key to match elements against.
	 * @return The number of elements removed.
	 */
	int32 Remove(KeyInitType Key)
	{
		if (Elements.Num())
		{
			return RemoveImpl(KeyFuncs::GetKeyHash(Key), Key);
		}

		return 0;
	}

	/**
	 * Removes an element from the set while maintaining set order.
	 * @param Element - A pointer to the element in the set, as returned by Add or Find.
	 */
	int32 RemoveStable(KeyInitType Key)
	{
		int32 Result = 0;

		if (Elements.Num())
		{
			Result = RemoveImpl(KeyFuncs::GetKeyHash(Key), Key);
			CompactStable();
		}

		return Result;
	}

	/**
	 * Removes all elements from the set matching the specified key.
	 *
	 * @see		Class documentation section on ByHash() functions
	 * @param	Key - The key to match elements against.
	 * @return	The number of elements removed.
	 */
	template<typename ComparableKey>
	int32 RemoveByHash(uint32 KeyHash, const ComparableKey& Key)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Disable deprecations warnings to stop warnings being thrown by our check macro.
		checkSlow(KeyHash == KeyFuncs::GetKeyHash(Key));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (Elements.Num())
		{
			return RemoveImpl(KeyHash, Key);
		}

		return 0;
	}

	/**
	 * Checks if the element contains an element with the given key.
	 * @param Key - The key to check for.
	 * @return true if the set contains an element with the given key.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool Contains(KeyInitType Key) const
	{
		return FindIndexByHash(KeyFuncs::GetKeyHash(Key), Key) != INDEX_NONE;
	}

	/**
	 * Checks if the element contains an element with the given key.
	 *
	 * @see	Class documentation section on ByHash() functions
	 */
	template<typename ComparableKey>
	[[nodiscard]] inline bool ContainsByHash(uint32 KeyHash, const ComparableKey& Key) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Disable deprecations warnings to stop warnings being thrown by our check macro.
		checkSlow(KeyHash == KeyFuncs::GetKeyHash(Key));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		return FindIndexByHash(KeyHash, Key) != INDEX_NONE;
	}

	/**
	 * Sorts the set's elements using the provided comparison class.
	 */
	template <typename PREDICATE_CLASS>
	void Sort( const PREDICATE_CLASS& Predicate )
	{
		// Sort the elements according to the provided comparison class.
		Elements.Sort( FElementCompareClass< PREDICATE_CLASS >( Predicate ) );

		// Rehash.
		Rehash();
	}

	/**
	 * Stable sorts the set's elements using the provided comparison class.
	 */
	template <typename PREDICATE_CLASS>
	void StableSort(const PREDICATE_CLASS& Predicate)
	{
		// Sort the elements according to the provided comparison class.
		Elements.StableSort(FElementCompareClass< PREDICATE_CLASS >(Predicate));

		// Rehash.
		Rehash();
	}

	/**
	* Sort the free element list so that subsequent additions will occur in the lowest available
	* TSparseArray index resulting in tighter packing without moving any existing items. Also useful for
	* some types of determinism. @see TSparseArray::SortFreeList() for more info.
	*/
	void SortFreeList()
	{
		Elements.SortFreeList();
	}

	/**
	 * Describes the set's contents through an output device.
	 * @param Ar - The output device to describe the set's contents through.
	 */
	void Dump(FOutputDevice& Ar)
	{
		Ar.Logf( TEXT("UE_TSPARSE_SET: %i elements, %i hash slots"), Elements.Num(), HashSize );
		for (int32 HashIndex = 0, LocalHashSize = HashSize; HashIndex < LocalHashSize; ++HashIndex)
		{
			// Count the number of elements in this hash bucket.
			int32 NumElementsInBucket = 0;
			for(FSetElementId ElementId = GetTypedHash(HashIndex);
				ElementId.IsValidId();
				ElementId = Elements[ElementId.AsInteger()].HashNextId)
			{
				NumElementsInBucket++;
			}

			Ar.Logf(TEXT("   Hash[%i] = %i"),HashIndex,NumElementsInBucket);
		}
	}

	[[nodiscard]] bool VerifyHashElementsKey(KeyInitType Key) const
	{
		bool bResult=true;
		if (Elements.Num())
		{
			// iterate over all elements for the hash entry of the given key
			// and verify that the ids are valid
			FSetElementId ElementId = GetTypedHash(KeyFuncs::GetKeyHash(Key));
			while( ElementId.IsValidId() )
			{
				if( !IsValidId(ElementId) )
				{
					bResult=false;
					break;
				}
				ElementId = Elements[ElementId.AsInteger()].HashNextId;
			}
		}
		return bResult;
	}

	void DumpHashElements(FOutputDevice& Ar)
	{
		for (int32 HashIndex = 0, LocalHashSize = HashSize; HashIndex < LocalHashSize; ++HashIndex)
		{
			Ar.Logf(TEXT("   Hash[%i]"),HashIndex);

			// iterate over all elements for the all hash entries
			// and dump info for all elements
			FSetElementId ElementId = GetTypedHash(HashIndex);
			while( ElementId.IsValidId() )
			{
				if( !IsValidId(ElementId) )
				{
					Ar.Logf(TEXT("		!!INVALID!! ElementId = %d"),ElementId.AsInteger());
				}
				else
				{
					Ar.Logf(TEXT("		VALID ElementId = %d"),ElementId.AsInteger());
				}
				ElementId = Elements[ElementId].HashNextId;
			}
		}
	}

	/** @return the intersection of two sets. (A AND B)*/
	[[nodiscard]] UE_TSPARSE_SET Intersect(const UE_TSPARSE_SET& OtherSet) const
	{
		const bool bOtherSmaller = (Num() > OtherSet.Num());
		const UE_TSPARSE_SET& A = (bOtherSmaller ? OtherSet : *this);
		const UE_TSPARSE_SET& B = (bOtherSmaller ? *this : OtherSet);

		UE_TSPARSE_SET Result;
		Result.Reserve(A.Num()); // Worst case is everything in smaller is in larger

		for(TConstIterator SetIt(A);SetIt;++SetIt)
		{
			if(B.Contains(KeyFuncs::GetSetKey(*SetIt)))
			{
				Result.Add(*SetIt);
			}
		}
		return Result;
	}

	/** @return the union of two sets. (A OR B)*/
	[[nodiscard]] UE_TSPARSE_SET Union(const UE_TSPARSE_SET& OtherSet) const
	{
		UE_TSPARSE_SET Result;
		Result.Reserve(Num() + OtherSet.Num()); // Worst case is 2 totally unique Sets

		for(TConstIterator SetIt(*this);SetIt;++SetIt)
		{
			Result.Add(*SetIt);
		}
		for(TConstIterator SetIt(OtherSet);SetIt;++SetIt)
		{
			Result.Add(*SetIt);
		}
		return Result;
	}

	/** @return the complement of two sets. (A not in B where A is this and B is Other)*/
	[[nodiscard]] UE_TSPARSE_SET Difference(const UE_TSPARSE_SET& OtherSet) const
	{
		UE_TSPARSE_SET Result;
		Result.Reserve(Num()); // Worst case is no elements of this are in Other

		for(TConstIterator SetIt(*this);SetIt;++SetIt)
		{
			if(!OtherSet.Contains(KeyFuncs::GetSetKey(*SetIt)))
			{
				Result.Add(*SetIt);
			}
		}
		return Result;
	}
	
	/** @return the symmetric difference of two sets. (A not in B and B not in A where A is this and B is Other) */
	[[nodiscard]] UE_TSPARSE_SET SymmetricDifference(const UE_TSPARSE_SET& OtherSet) const
	{
		UE_TSPARSE_SET Result;
		Result.Reserve(Num() + OtherSet.Num()); // Worst case is no overlap

		// Add elements from A that are not in B
		for(TConstIterator SetIt(*this);SetIt;++SetIt)
		{
			if(!OtherSet.Contains(KeyFuncs::GetSetKey(*SetIt)))
			{
				Result.Add(*SetIt);
			}
		}

		// Add elements from B that are not in A
		for(TConstIterator SetIt(OtherSet);SetIt;++SetIt)
		{
			if(!this->Contains(KeyFuncs::GetSetKey(*SetIt)))
			{
				Result.Add(*SetIt);
			}
		}

		return Result;
	}

	/**
	 * Determine whether the specified set is entirely included within this set
	 *
	 * @param OtherSet	Set to check
	 *
	 * @return True if the other set is entirely included in this set, false if it is not
	 */
	[[nodiscard]] bool Includes(const UE_TSPARSE_SET<ElementType,KeyFuncs,Allocator>& OtherSet) const
	{
		bool bIncludesSet = true;
		if (OtherSet.Num() <= Num())
		{
			for(TConstIterator OtherSetIt(OtherSet); OtherSetIt; ++OtherSetIt)
			{
				if (!Contains(KeyFuncs::GetSetKey(*OtherSetIt)))
				{
					bIncludesSet = false;
					break;
				}
			}
		}
		else
		{
			// Not possible to include if it is bigger than us
			bIncludesSet = false;
		}
		return bIncludesSet;
	}

	/** @return a TArray of the elements */
	[[nodiscard]] TArray<ElementType> Array() const
	{
		TArray<ElementType> Result;
		Result.Reserve(Num());
		for(TConstIterator SetIt(*this);SetIt;++SetIt)
		{
			Result.Add(*SetIt);
		}
		return Result;
	}

	/**
	 * Checks that the specified address is not part of an element within the container.  Used for implementations
	 * to check that reference arguments aren't going to be invalidated by possible reallocation.
	 *
	 * @param Addr The address to check.
	 */
	UE_FORCEINLINE_HINT void CheckAddress(const ElementType* Addr) const
	{
		Elements.CheckAddress(Addr);
	}

	/**
	 * Move assignment operator.
	 * Compatible element type version.
	 *
	 * @param Other Set to assign and move from.
	 */
	template <
		typename OtherKeyFuncs,
		typename AliasElementType = ElementType
		UE_REQUIRES(TIsContainerElementTypeCopyable_V<AliasElementType>)
	>
	UE_TSPARSE_SET& operator=(UE_TSPARSE_SET<typename TContainerElementTypeCompatibility<ElementType>::CopyFromOtherType, OtherKeyFuncs, Allocator>&& Other)
	{
		TContainerElementTypeCompatibility<ElementType>::CopyingFromOtherType();
		Reset();
		Append(MoveTemp(Other));
		return *this;
	}

	/**
	 * Assignment operator. First deletes all currently contained elements
	 * and then copies from other set.
	 * Compatible element type version.
	 *
	 * @param Other The source set to assign from.
	 */
	template <
		typename OtherKeyFuncs,
		typename OtherAllocator,
		typename AliasElementType = ElementType
		UE_REQUIRES(TIsContainerElementTypeCopyable_V<AliasElementType>)
	>
	UE_TSPARSE_SET& operator=(const UE_TSPARSE_SET<typename TContainerElementTypeCompatibility<ElementType>::CopyFromOtherType, OtherKeyFuncs, OtherAllocator>& Other)
	{
		TContainerElementTypeCompatibility<ElementType>::CopyingFromOtherType();
		Reset();
		Append(Other);
		return *this;
	}

	/**
	 * Add all items from another set to our set (union without creating a new set)
	 * Compatible element type version.
	 * @param OtherSet - The other set of items to add.
	 */
	template <
		typename OtherKeyFuncs,
		typename OtherAllocator,
		typename AliasElementType = ElementType
		UE_REQUIRES(TIsContainerElementTypeCopyable_V<AliasElementType>)
	>
	void Append(const UE_TSPARSE_SET<typename TContainerElementTypeCompatibility<ElementType>::CopyFromOtherType, OtherKeyFuncs, OtherAllocator>& OtherSet)
	{
		TContainerElementTypeCompatibility<ElementType>::CopyingFromOtherType();
		Reserve(Elements.Num() + OtherSet.Num());
		for (const ElementType& Element : OtherSet)
		{
			Add(Element);
		}
	}

	/**
	 * Add all items from another set to our set (union without creating a new set)
	 * Compatible element type version.
	 * @param OtherSet - The other set of items to add.
	 */
	template <
		typename OtherKeyFuncs,
		typename AliasElementType = ElementType
		UE_REQUIRES(TIsContainerElementTypeCopyable_V<AliasElementType>)
	>
	void Append(UE_TSPARSE_SET<typename TContainerElementTypeCompatibility<ElementType>::CopyFromOtherType, OtherKeyFuncs, Allocator>&& OtherSet)
	{
		TContainerElementTypeCompatibility<ElementType>::CopyingFromOtherType();
		Reserve(Elements.Num() + OtherSet.Num());
		for (ElementType& Element : OtherSet)
		{
			Add(MoveTempIfPossible(Element));
		}
		OtherSet.Reset();
	}

private:
	/** Extracts the element value from the set's element structure and passes it to the user provided comparison class. */
	template <typename PREDICATE_CLASS>
	class FElementCompareClass
	{
		TDereferenceWrapper< ElementType, PREDICATE_CLASS > Predicate;

	public:
		[[nodiscard]] UE_FORCEINLINE_HINT FElementCompareClass( const PREDICATE_CLASS& InPredicate )
			: Predicate( InPredicate )
		{
		}

		[[nodiscard]] UE_FORCEINLINE_HINT bool operator()( const SetElementType& A,const SetElementType& B ) const
		{
			return Predicate( A.Value, B.Value );
		}
	};

	using ElementArrayType = TSparseArray<SetElementType, typename Allocator::SparseArrayAllocator>;
	using HashType         = typename Allocator::HashAllocator::template ForElementType<FSetElementId>;

	ElementArrayType Elements;

	HashType Hash;
	int32	 HashSize = 0;

public:
	void WriteMemoryImage(FMemoryImageWriter& Writer) const
	{
		checkf(!Writer.Is32BitTarget(), TEXT("UE_TSPARSE_SET does not currently support freezing for 32bits"));
		if constexpr (TAllocatorTraits<Allocator>::SupportsFreezeMemoryImage && THasTypeLayout<InElementType>::Value)
		{
			this->Elements.WriteMemoryImage(Writer);
			this->Hash.WriteMemoryImage(Writer, StaticGetTypeLayoutDesc<FSetElementId>(), this->HashSize);
			Writer.WriteBytes(this->HashSize);
		}
		else
		{
			Writer.WriteBytes(UE_TSPARSE_SET());
		}
	}

	void CopyUnfrozen(const FMemoryUnfreezeContent& Context, void* Dst) const
	{
		if constexpr (TAllocatorTraits<Allocator>::SupportsFreezeMemoryImage && THasTypeLayout<InElementType>::Value)
		{
			UE_TSPARSE_SET* DstObject = static_cast<UE_TSPARSE_SET*>(Dst);
			this->Elements.CopyUnfrozen(Context, &DstObject->Elements);

			::new((void*)&DstObject->Hash) HashType();
			DstObject->Hash.ResizeAllocation(0, this->HashSize, sizeof(FSetElementId));
			FMemory::Memcpy(DstObject->Hash.GetAllocation(), this->Hash.GetAllocation(), sizeof(FSetElementId) * this->HashSize);
			DstObject->HashSize = this->HashSize;
		}
		else
		{
			::new(Dst) UE_TSPARSE_SET();
		}
	}

	static void AppendHash(const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		ElementArrayType::AppendHash(LayoutParams, Hasher);
	}

private:

	[[nodiscard]] UE_FORCEINLINE_HINT FSetElementId& GetTypedHash(int32 HashIndex) const
	{
		return ((FSetElementId*)Hash.GetAllocation())[HashIndex & (HashSize - 1)];
	}

	/** Links an added element to the hash chain. */
	inline void LinkElement(SizeType ElementIndex, const SetElementType& Element, uint32 KeyHash) const
	{
		// Compute the hash bucket the element goes in.
		Element.HashIndex = KeyHash & (HashSize - 1);

		// Link the element into the hash bucket.
		FSetElementId& TypedHash = GetTypedHash(Element.HashIndex);
		Element.HashNextId = TypedHash;
		TypedHash = FSetElementId::FromInteger(ElementIndex);
	}

	/** Hashes and links an added element to the hash chain. */
	UE_FORCEINLINE_HINT void HashElement(SizeType ElementIndex, const SetElementType& Element) const
	{
		LinkElement(ElementIndex, Element, KeyFuncs::GetKeyHash(KeyFuncs::GetSetKey(Element.Value)));
	}

	/** Reset hash buckets to invalid */
	void UnhashElements()
	{
		FSetElementId* HashPtr = Hash.GetAllocation();

		// Check if it should be faster to clear the hash by going through elements instead of resetting the whole hash
		if (Num() < (HashSize / UE_SET_UNHASH_THRESHOLD_STORED_KEY))
		{
			// Faster path: only reset hash buckets to invalid for elements in the hash
			for (const SetElementType& Element: Elements)
			{
				HashPtr[Element.HashIndex] = FSetElementId();
			}
		}
		else
		{
			static_assert(FSetElementId().AsInteger() == -1);
			FMemory::Memset(HashPtr, 0xFF, HashSize * sizeof(FSetElementId));
		}
	}

	/**
	 * Checks if the hash has an appropriate number of buckets, and if it should be resized.
	 * @param NumHashedElements - The number of elements to size the hash for.
	 * @param DesiredHashSize - Desired size if we should rehash.
	 * @param AllowShrinking - If the hash is allowed to shrink.
	 * @return true if the set should berehashed.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool ShouldRehash(int32 NumHashedElements, int32 DesiredHashSize, EAllowShrinking AllowShrinking) const
	{
		// If the hash hasn't been created yet, or is smaller than the desired hash size, rehash.
		// If shrinking is allowed and the hash is bigger than the desired hash size, rehash.
		return ((NumHashedElements > 0 && HashSize < DesiredHashSize) || (AllowShrinking == EAllowShrinking::Yes && HashSize > DesiredHashSize));
	}

	/**
	 * Checks if the hash has an appropriate number of buckets, and if not resizes it.
	 * @param NumHashedElements - The number of elements to size the hash for.
	 * @param AllowShrinking - If the hash is allowed to shrink.
	 * @return true if the set was rehashed.
	 */
	bool ConditionalRehash(int32 NumHashedElements, EAllowShrinking AllowShrinking)
	{
		// Calculate the desired hash size for the specified number of elements.
		const int32 DesiredHashSize = Allocator::GetNumberOfHashBuckets(NumHashedElements);

		if (ShouldRehash(NumHashedElements, DesiredHashSize, AllowShrinking))
		{
			HashSize = DesiredHashSize;
			Rehash();
			return true;
		}

		return false;
	}

	/** Resizes the hash. */
	void Rehash()
	{
		UE::Core::Private::Rehash(Hash, HashSize);

		if (HashSize)
		{
			// Add the existing elements to the new hash.
			for(typename ElementArrayType::TConstIterator ElementIt(Elements);ElementIt;++ElementIt)
			{
				HashElement(ElementIt.GetIndex(), *ElementIt);
			}
		}
	}

	/** The base type of whole set iterators. */
	template<bool bConst, bool bRangedFor = false>
	class TBaseIterator
	{
	private:
		using SetType = std::conditional_t<bConst, const UE_TSPARSE_SET, UE_TSPARSE_SET>;
		using ItElementType = std::conditional_t<bConst,const ElementType,ElementType>;

	public:
		typedef std::conditional_t<
			bConst,
			std::conditional_t<bRangedFor, typename ElementArrayType::TRangedForConstIterator, typename ElementArrayType::TConstIterator>,
			std::conditional_t<bRangedFor, typename ElementArrayType::TRangedForIterator,      typename ElementArrayType::TIterator     >
		> ElementItType;

		[[nodiscard]] UE_FORCEINLINE_HINT explicit TBaseIterator(SetType& InSet)
			: ElementIt(InSet.Elements.begin())
			, Set(InSet)
#if DO_SPARSE_SET_ITERATOR_CHECKS
			, InitialNum(InSet.Num())
#endif
		{
		}

		[[nodiscard]] UE_FORCEINLINE_HINT explicit TBaseIterator(SetType& InSet, const ElementItType& InElementIt)
			: ElementIt(InElementIt)
			, Set(InSet)
#if DO_SPARSE_SET_ITERATOR_CHECKS
			, InitialNum(InSet.Num())
#endif
		{
		}

		/** Advances the iterator to the next element. */
		UE_FORCEINLINE_HINT TBaseIterator& operator++()
		{
#if DO_SPARSE_SET_ITERATOR_CHECKS
			const int32 SetNum = Set.Num();
			ensureMsgf(SetNum >= InitialNum, TEXT("Sets/Maps should never have elements removed during iteration outside of Iterator.RemoveCurrent(). InitialNum %d CurrentNum %d"), InitialNum, SetNum);
			LastRemoved = INDEX_NONE;
#endif

			++ElementIt;
			return *this;
		}

		/** conversion to "bool" returning true if the iterator is valid. */
		[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
		{ 
			return !!ElementIt; 
		}
		/** inverse of the "bool" operator */
		[[nodiscard]] UE_FORCEINLINE_HINT bool operator!() const 
		{
			return !(bool)*this;
		}

		// Accessors.
		[[nodiscard]] UE_FORCEINLINE_HINT FSetElementId GetId() const
		{
			return FSetElementId::FromInteger(ElementIt.GetIndex());
		}
		[[nodiscard]] UE_FORCEINLINE_HINT ItElementType* operator->() const
		{
#if DO_SPARSE_SET_ITERATOR_CHECKS
			checkf(LastRemoved == INDEX_NONE, TEXT("Accessing iterator after it was removed from the set and before it was incremented to the next valid elements."));
#endif
			return &ElementIt->Value;
		}
		[[nodiscard]] UE_FORCEINLINE_HINT ItElementType& operator*() const
		{
#if DO_SPARSE_SET_ITERATOR_CHECKS
			checkf(LastRemoved == INDEX_NONE, TEXT("Accessing iterator after it was removed from the set and before it was incremented to the next valid elements."));
#endif
			return ElementIt->Value;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TBaseIterator& Rhs) const { return ElementIt != Rhs.ElementIt; }
		[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TBaseIterator& Rhs) const { return ElementIt == Rhs.ElementIt; }

	protected:
		ElementItType ElementIt;
		SetType& Set;

#if DO_SPARSE_SET_ITERATOR_CHECKS
		SizeType InitialNum;
		SizeType LastRemoved = INDEX_NONE;
#endif
	};

	/** The base type of whole set iterators. */
	template<bool bConst>
	class TBaseKeyIterator
	{
	private:
		using SetType = std::conditional_t<bConst, const UE_TSPARSE_SET, UE_TSPARSE_SET>;
		using ItElementType = std::conditional_t<bConst,const ElementType,ElementType>;
		using ReferenceOrValueType = typename TTypeTraits<typename KeyFuncs::KeyType>::ConstPointerType;

	public:
		using KeyArgumentType =
			std::conditional_t<
				std::is_reference_v<ReferenceOrValueType>,
				TRetainedRef<std::remove_reference_t<ReferenceOrValueType>>,
				KeyInitType
			>;

		/** Initialization constructor. */
		[[nodiscard]] inline explicit TBaseKeyIterator(SetType& InSet, KeyArgumentType InKey)
			: Set  (InSet)
			, Key  (InKey) //-V1041
			, Index(INDEX_NONE)
#if DO_SPARSE_SET_ITERATOR_CHECKS
			, InitialNum(InSet.Num())
#endif
		{
			if (Set.HashSize)
			{
				NextIndex = Set.GetTypedHash(KeyFuncs::GetKeyHash(Key)).AsInteger();
				++(*this);
			}
			else
			{
				NextIndex = INDEX_NONE;
			}
		}

		/** Advances the iterator to the next element. */
		inline TBaseKeyIterator& operator++()
		{
			// Note: Adding new elements is safe, however they will be guaranteed to be missed by the current iteration
#if DO_SPARSE_SET_ITERATOR_CHECKS
			const int32 SetNum = Set.Num();
			ensureMsgf(SetNum >= InitialNum, TEXT("Sets/Maps should never have elements removed during iteration outside of Iterator.RemoveCurrent(). InitialNum %d CurrentNum %d"), InitialNum, SetNum);
			LastRemoved = INDEX_NONE;
#endif

			Index = NextIndex;

			while (Index != INDEX_NONE)
			{
				NextIndex = Set.Elements[Index].HashNextId.AsInteger();
				checkSlow(Index != NextIndex);

				if (KeyFuncs::Matches(KeyFuncs::GetSetKey(Set.Elements[Index].Value),Key))
				{
					break;
				}

				Index = NextIndex;
			}
			return *this;
		}

		/** conversion to "bool" returning true if the iterator is valid. */
		[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
		{ 
			return Index != INDEX_NONE;
		}
		/** inverse of the "bool" operator */
		[[nodiscard]] UE_FORCEINLINE_HINT bool operator!() const
		{
			return !(bool)*this;
		}

		// Accessors.
		[[nodiscard]] UE_FORCEINLINE_HINT FSetElementId GetId() const
		{
			return FSetElementId::FromInteger(Index);
		}
		[[nodiscard]] UE_FORCEINLINE_HINT ItElementType* operator->() const
		{
			return &Set.Elements[Index].Value;
		}
		[[nodiscard]] UE_FORCEINLINE_HINT ItElementType& operator*() const
		{
			return Set.Elements[Index].Value;
		}

	protected:
		SetType& Set;
		ReferenceOrValueType Key;
		SizeType Index;
		SizeType NextIndex;

#if DO_SPARSE_SET_ITERATOR_CHECKS
		SizeType InitialNum;
		SizeType LastRemoved = INDEX_NONE;
#endif
	};

public:

	/** Used to iterate over the elements of a const UE_TSPARSE_SET. */
	using TConstIterator = TBaseIterator<true>;

	/** Used to iterate over the elements of a UE_TSPARSE_SET. */
	class TIterator : public TBaseIterator<false>
	{
		friend class UE_TSPARSE_SET;
		using Super = TBaseIterator<false>;

	public:
		using Super::Super;

		/** Removes the current element from the set. */
		UE_FORCEINLINE_HINT void RemoveCurrent()
		{
#if DO_SPARSE_SET_ITERATOR_CHECKS
			ensureMsgf(this->LastRemoved == INDEX_NONE, TEXT("RemoveCurrent was already called for the current iteration index of %d"), this->LastRemoved);
			this->LastRemoved = this->ElementIt.GetIndex();
			--this->InitialNum;
#endif

			this->Set.RemoveByIndex(this->ElementIt.GetIndex());
		}
	};

	using TRangedForConstIterator = TBaseIterator<true, true>;
	using TRangedForIterator      = TBaseIterator<false, true>;

	/** Used to iterate over the elements of a const UE_TSPARSE_SET. */
	using TConstKeyIterator = TBaseKeyIterator<true>;

	/** Used to iterate over the elements of a UE_TSPARSE_SET. */
	class TKeyIterator : public TBaseKeyIterator<false>
	{
	private:
		using Super = TBaseKeyIterator<false>;

	public:
		using KeyArgumentType = typename Super::KeyArgumentType;

		using Super::Super;

		/** Removes the current element from the set. */
		inline void RemoveCurrent()
		{
#if DO_SPARSE_SET_ITERATOR_CHECKS
			ensureMsgf(this->LastRemoved == INDEX_NONE, TEXT("RemoveCurrent was already called for the current iteration index of %d"), this->LastRemoved);
			this->LastRemoved = this->Index;
			--this->InitialNum;
#endif

			this->Set.RemoveByIndex(this->Index);
			this->Index = INDEX_NONE;
		}
	};

	/** Creates an iterator for the contents of this set */
	[[nodiscard]] UE_FORCEINLINE_HINT TIterator CreateIterator()
	{
		return TIterator(*this);
	}

	/** Creates a const iterator for the contents of this set */
	[[nodiscard]] UE_FORCEINLINE_HINT TConstIterator CreateConstIterator() const
	{
		return TConstIterator(*this);
	}

	friend struct TSETPRIVATEFRIEND;

public:
	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForIterator begin()
	{
		return TRangedForIterator(*this);
	}
	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForConstIterator begin() const
	{
		return TRangedForConstIterator(*this);
	}
	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForIterator end()
	{
		return TRangedForIterator(*this, Elements.end());
	}
	[[nodiscard]] UE_FORCEINLINE_HINT TRangedForConstIterator end() const
	{
		return TRangedForConstIterator(*this, Elements.end());
	}

	// Maps are deliberately prevented from being hashed or compared, because this would hide potentially major performance problems behind default operations.
	friend uint32 GetTypeHash(const UE_TSPARSE_SET& Set) = delete;
	friend bool operator==(const UE_TSPARSE_SET&, const UE_TSPARSE_SET&) = delete;
	friend bool operator!=(const UE_TSPARSE_SET&, const UE_TSPARSE_SET&) = delete;
};

template <typename RangeType>
UE_TSPARSE_SET(RangeType&&) -> UE_TSPARSE_SET<TElementType_T<RangeType>>;

namespace Freeze
{
	template<typename ElementType, typename KeyFuncs, typename Allocator>
	void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>& Object, const FTypeLayoutDesc&)
	{
		Object.WriteMemoryImage(Writer);
	}

	template<typename ElementType, typename KeyFuncs, typename Allocator>
	uint32 IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>& Object, void* OutDst)
	{
		Object.CopyUnfrozen(Context, OutDst);
		return sizeof(Object);
	}

	template<typename ElementType, typename KeyFuncs, typename Allocator>
	uint32 IntrinsicAppendHash(const UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>::AppendHash(LayoutParams, Hasher);
		return DefaultAppendHash(TypeDesc, LayoutParams, Hasher);
	}
}

DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT((template <typename ElementType, typename KeyFuncs, typename Allocator>), (UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>));

struct TSETPRIVATEFRIEND
{
	/** Serializer. */
	template<typename ElementType, typename KeyFuncs,typename Allocator>
	static FArchive& Serialize(FArchive& Ar,UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>& Set)
	{
		// Load the set's new elements.
		Ar << Set.Elements;

		if(Ar.IsLoading() || (Ar.IsModifyingWeakAndStrongReferences() && !Ar.IsSaving()))
		{
			// Free the old hash.
			Set.Hash.ResizeAllocation(0,0,sizeof(FSetElementId));
			Set.HashSize = 0;

			// Hash the newly loaded elements.
			Set.ConditionalRehash(Set.Elements.Num(), EAllowShrinking::No);
		}

		return Ar;
	}

	/** Structured archive serializer. */
	template<typename ElementType, typename KeyFuncs,typename Allocator>
 	static void SerializeStructured(FStructuredArchive::FSlot Slot, UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>& Set)
 	{
		Slot << Set.Elements;

		if (Slot.GetUnderlyingArchive().IsLoading() || (Slot.GetUnderlyingArchive().IsModifyingWeakAndStrongReferences() && !Slot.GetUnderlyingArchive().IsSaving()))
		{
			// Free the old hash.
			Set.Hash.ResizeAllocation(0, 0, sizeof(FSetElementId));
			Set.HashSize = 0;

			// Hash the newly loaded elements.
			Set.ConditionalRehash(Set.Elements.Num(), EAllowShrinking::No);
		}
 	}

	// Legacy comparison operators.  Note that these also test whether the set's elements were added in the same order!
	template<typename ElementType, typename KeyFuncs,typename Allocator>
	[[nodiscard]] static bool LegacyCompareEqual(const UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>& A, const UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>& B)
	{
		return A.Elements == B.Elements;
	}
};

/** Serializer. */
template<typename ElementType, typename KeyFuncs,typename Allocator>
FArchive& operator<<(FArchive& Ar, UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>& Set)
{
	return TSETPRIVATEFRIEND::Serialize(Ar, Set);
}

/** Structured archive serializer. */
template<typename ElementType, typename KeyFuncs,typename Allocator>
void operator<<(FStructuredArchive::FSlot& Ar, UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>& Set)
{
	TSETPRIVATEFRIEND::SerializeStructured(Ar, Set);
}

// Legacy comparison operators.  Note that these also test whether the set's elements were added in the same order!
template<typename ElementType, typename KeyFuncs,typename Allocator>
[[nodiscard]] bool LegacyCompareEqual(const UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>& A,const UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>& B)
{
	return TSETPRIVATEFRIEND::LegacyCompareEqual(A, B);
}
template<typename ElementType, typename KeyFuncs,typename Allocator>
[[nodiscard]] bool LegacyCompareNotEqual(const UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>& A,const UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>& B)
{
	return !TSETPRIVATEFRIEND::LegacyCompareEqual(A, B);
}

template <typename ElementType, typename KeyFuncs, typename Allocator>
inline constexpr bool TIsTSet_V<UE_TSPARSE_SET<ElementType, KeyFuncs, Allocator>> = true;

#undef TSETPRIVATEFRIEND
