// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SetUtilities.h"
#include "Serialization/StructuredArchive.h"
#include <initializer_list>
#include <type_traits>

template<typename InElementType, bool bTypeLayout>
class TSparseSetElementBase
{
public:
	typedef InElementType ElementType;

	[[nodiscard]] TSparseSetElementBase() = default;

	/** Initialization constructor. */
	template <
		typename... InitType
		UE_REQUIRES((sizeof...(InitType) != 1) || (!std::is_same_v<TSparseSetElementBase, std::decay_t<InitType>> && ...))
	>
	[[nodiscard]] explicit UE_FORCEINLINE_HINT TSparseSetElementBase(InitType&&... InValue)
		: Value(Forward<InitType>(InValue)...)
	{
	}

	[[nodiscard]] TSparseSetElementBase(TSparseSetElementBase&&) = default;
	[[nodiscard]] TSparseSetElementBase(const TSparseSetElementBase&) = default;
	TSparseSetElementBase& operator=(TSparseSetElementBase&&) = default;
	TSparseSetElementBase& operator=(const TSparseSetElementBase&) = default;

	/** The element's value. */
	ElementType Value;

	/** The id of the next element in the same hash bucket. */
	mutable FSetElementId HashNextId;

	/** The hash bucket that the element is currently linked to. */
	mutable int32 HashIndex;
};

template<typename InElementType>
class TSparseSetElementBase<InElementType, true>
{
	DECLARE_INLINE_TYPE_LAYOUT(TSparseSetElementBase, NonVirtual);
public:
	typedef InElementType ElementType;

	[[nodiscard]] TSparseSetElementBase() = default;

	/** Initialization constructor. */
	template <
		typename... InitType
		UE_REQUIRES((sizeof...(InitType) != 1) || (!std::is_same_v<TSparseSetElementBase, std::decay_t<InitType>> && ...))
	>
	[[nodiscard]] explicit UE_FORCEINLINE_HINT TSparseSetElementBase(InitType&&... InValue)
		: Value(Forward<InitType>(InValue)...)
	{
	}

	[[nodiscard]] TSparseSetElementBase(TSparseSetElementBase&&) = default;
	[[nodiscard]] TSparseSetElementBase(const TSparseSetElementBase&) = default;
	TSparseSetElementBase& operator=(TSparseSetElementBase&&) = default;
	TSparseSetElementBase& operator=(const TSparseSetElementBase&) = default;

	/** The element's value. */
	LAYOUT_FIELD(ElementType, Value);

	/** The id of the next element in the same hash bucket. */
	LAYOUT_MUTABLE_FIELD(FSetElementId, HashNextId);

	/** The hash bucket that the element is currently linked to. */
	LAYOUT_MUTABLE_FIELD(int32, HashIndex);
};

/** An element in the set. */
template <typename InElementType>
class TSparseSetElement : public TSparseSetElementBase<InElementType, THasTypeLayout<InElementType>::Value>
{
	using Super = TSparseSetElementBase<InElementType, THasTypeLayout<InElementType>::Value>;
public:
	/** Default constructor. */
	[[nodiscard]] TSparseSetElement() = default;

	/** Initialization constructor. */
	template <
	typename... InitType
	UE_REQUIRES((sizeof...(InitType) != 1) || (!std::is_same_v<TSparseSetElement, std::decay_t<InitType>> && ...))
	>
	[[nodiscard]] explicit UE_FORCEINLINE_HINT TSparseSetElement(InitType&&... InValue)
	: Super(Forward<InitType>(InValue)...)
	{
	}

	[[nodiscard]] TSparseSetElement(TSparseSetElement&&) = default;
	[[nodiscard]] TSparseSetElement(const TSparseSetElement&) = default;
	TSparseSetElement& operator=(TSparseSetElement&&) = default;
	TSparseSetElement& operator=(const TSparseSetElement&) = default;

	// Comparison operators
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TSparseSetElement& Other) const
	{
		return this->Value == Other.Value;
	}
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TSparseSetElement& Other) const
	{
		return this->Value != Other.Value;
	}
};

namespace UE::Core::Private
{
	[[noreturn]] CORE_API void OnInvalidSetNum(unsigned long long NewNum);

	template<typename HashType>
	void CopyHash(HashType& Hash, int32& HashSize, const HashType& Copy, int32 HashSizeCopy)
	{
		DestructItems((FSetElementId*)Hash.GetAllocation(), HashSize);
		Hash.ResizeAllocation(0, HashSizeCopy, sizeof(FSetElementId));
		ConstructItems<FSetElementId>(Hash.GetAllocation(), (FSetElementId*)Copy.GetAllocation(), HashSizeCopy);
		HashSize = HashSizeCopy;
	}

	template<typename HashType>
	[[nodiscard]] UE_FORCEINLINE_HINT FSetElementId& GetTypedHash(HashType& Hash, int32 HashIndex, int32 HashSize)
	{
		return ((FSetElementId*)Hash.GetAllocation())[HashIndex & (HashSize - 1)];
	}

	template<typename HashType>
	void Rehash(HashType& Hash, int32 HashSize)
	{
		// Free the old hash.
		Hash.ResizeAllocation(0, 0, sizeof(FSetElementId));

		if (HashSize)
		{
			// Allocate the new hash.
			checkSlow(FMath::IsPowerOfTwo(HashSize));
			Hash.ResizeAllocation(0, HashSize, sizeof(FSetElementId));
			for (int32 HashIndex = 0; HashIndex < HashSize; ++HashIndex)
			{
				GetTypedHash(Hash, HashIndex, HashSize) = FSetElementId();
			}
		}
	}
}

/** Serializer. */
template <typename ElementType>
UE_FORCEINLINE_HINT FArchive& operator<<(FArchive& Ar, TSparseSetElement<ElementType>& Element)
{
	return Ar << Element.Value;
}

/** Structured archive serializer. */
template <typename ElementType>
UE_FORCEINLINE_HINT void operator<<(FStructuredArchive::FSlot& Ar, TSparseSetElement<ElementType>& Element)
{
	Ar << Element.Value;
}
