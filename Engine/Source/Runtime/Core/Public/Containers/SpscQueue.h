// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "AutoRTFM/ReportHazard.h"
#include "Templates/MemoryOps.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/UnrealTemplate.h"
#include "Misc/Optional.h"
#include <atomic>

/**
 * Fast single-producer/single-consumer unbounded concurrent queue. Doesn't free memory until destruction but recycles consumed items.
 * Based on http://www.1024cores.net/home/lock-free-algorithms/queues/unbounded-spsc-queue
 */
template<typename T, typename AllocatorType = FMemory>
class TSpscQueue final
{
public:
	using ElementType = T;

	UE_NONCOPYABLE(TSpscQueue);

	[[nodiscard]] TSpscQueue() : TSpscQueue(::new(AllocatorType::Malloc(sizeof(FNode), alignof(FNode))) FNode)
	{
	}

	~TSpscQueue()
	{
		FNode* Node = First;
		FNode* LocalTail = Tail.load(std::memory_order_relaxed);

		// Delete all nodes which are the sentinel or unoccupied
		bool bContinue = false;
		do
		{
			FNode* Next = Node->Next.load(std::memory_order_relaxed);
			bContinue = Node != LocalTail;
			AllocatorType::Free(Node);
			Node = Next;
		} while (bContinue);

		// Delete all nodes which are occupied, destroying the element first
		while (Node != nullptr)
		{
			FNode* Next = Node->Next.load(std::memory_order_relaxed);
			DestructItem((ElementType*)&Node->Value);
			AllocatorType::Free(Node);
			Node = Next;
		}
	}

	template <typename... ArgTypes>
	void Enqueue(ArgTypes&&... Args)
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(SpscQueue);

		FNode* Node = AllocNode();
		::new((void*)&Node->Value) ElementType(Forward<ArgTypes>(Args)...);

		Head->Next.store(Node, std::memory_order_release);
		Head = Node;

		NumElems++;
	}

	// returns empty TOptional if queue is empty
	TOptional<ElementType> Dequeue()
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(SpscQueue);

		FNode* LocalTail = Tail.load(std::memory_order_relaxed);
		FNode* LocalTailNext = LocalTail->Next.load(std::memory_order_acquire);
		if (LocalTailNext == nullptr)
		{
			return {};
		}

		ElementType* TailNextValue = (ElementType*)&LocalTailNext->Value;
		TOptional<ElementType> Value{ MoveTemp(*TailNextValue) };
		DestructItem(TailNextValue);

		Tail.store(LocalTailNext, std::memory_order_release);

		if (Value.IsSet())
		{
			NumElems--;
		}
		return Value;
	}

	bool Dequeue(ElementType& OutElem)
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(SpscQueue);

		TOptional<ElementType> LocalElement = Dequeue();
		if (LocalElement.IsSet())
		{
			OutElem = MoveTempIfPossible(LocalElement.GetValue());
			return true;
		}
		
		return false;
	}

	[[nodiscard]] bool IsEmpty() const
	{
		return !NumElems;
	}

	[[nodiscard]] int32 Num() const
	{
		return NumElems;
	}

	// as there can be only one consumer, a consumer can safely "peek" the tail of the queue.
	// returns a pointer to the tail if the queue is not empty, nullptr otherwise
	// there's no overload with TOptional as it doesn't support references
	[[nodiscard]] ElementType* Peek() const
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(SpscQueue);

		FNode* LocalTail = Tail.load(std::memory_order_relaxed);
		FNode* LocalTailNext = LocalTail->Next.load(std::memory_order_acquire);

		if (LocalTailNext == nullptr)
		{
			return nullptr;
		}

		return (ElementType*)&LocalTailNext->Value;
	}

private:
	struct FNode
	{
		std::atomic<FNode*> Next{ nullptr };
		TTypeCompatibleBytes<ElementType> Value;
	};

	[[nodiscard]] explicit TSpscQueue(FNode* Node) : Tail(Node), Head(Node), First(Node), TailCopy(Node)
	{
	}

public:
	//
	// Allows the single consumer to iterate the contents of the queue without popping from it.
	// 
	// The single producer may continue to insert items in the queue while the consumer is iterating.
	// These new items may or may not be seen by the consumer, since the consumer might have finished
	// iterating before reaching those new elements.
	//
	struct FIterator
	{
		TSpscQueue::FNode* Current;

		FIterator(const TSpscQueue& Queue)
			: Current(Queue.Tail.load(std::memory_order_relaxed))
		{
			++(*this);
		}

		FIterator& operator++ ()
		{
			Current = Current->Next.load(std::memory_order_acquire);
			return *this;
		}

		bool operator== (std::nullptr_t) const
		{
			return Current == nullptr;
		}

		const ElementType& operator* () const
		{
			return Current->Value.GetUnchecked();
		}
	};

	FIterator begin() const
	{
		UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(SpscQueue);

		return FIterator(*this);
	}

	std::nullptr_t end() const
	{
		return nullptr;
	}

private:
	FNode* AllocNode()
	{
		// first tries to allocate node from internal node cache, 
		// if attempt fails, allocates node via ::operator new() 

		auto AllocFromCache = [this]()
		{
			FNode* Node = First;
			First = First->Next;
			Node->Next.store(nullptr, std::memory_order_relaxed);
			return Node;
		};

		if (First != TailCopy)
		{
			return AllocFromCache();
		}

		TailCopy = Tail.load(std::memory_order_acquire);
		if (First != TailCopy)
		{
			return AllocFromCache();
		}

		return ::new(AllocatorType::Malloc(sizeof(FNode), alignof(FNode))) FNode();
	}

private:
	// consumer part 
	// accessed mainly by consumer, infrequently by producer 
	std::atomic<FNode*> Tail; // tail of the queue

	// producer part 
	// accessed only by producer 
	FNode* Head; // head of the queue

	FNode* First; // last unused node (tail of node cache) 
	FNode* TailCopy; // helper (points somewhere between First and Tail)

	std::atomic<int32> NumElems = 0;
};