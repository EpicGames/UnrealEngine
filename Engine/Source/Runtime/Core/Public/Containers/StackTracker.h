// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Math/NumericLimits.h"

class FOutputDevice;
class FString;


/**
 * Stack tracker. Used to identify callstacks at any point in the codebase.
 */
struct FStackTracker
{
public:
	/** Maximum number of backtrace depth. */
	static constexpr int32 MAX_BACKTRACE_DEPTH = 50;
	/** Helper structure to capture callstack addresses and stack count. */
	struct FCallStack
	{
		/** Stack count, aka the number of calls to CalculateStack */
		int64 StackCount;
		/** Program counter addresses for callstack. */
		uint64 Addresses[MAX_BACKTRACE_DEPTH];
		/** User data to store with the stack trace for later use */
		void* UserData;
	};

	/** Used to optionally update the information currently stored with the callstack */
	typedef void (*StackTrackerUpdateFn)( const FCallStack& CallStack, void* UserData);
	/** Used to optionally report information based on the current stack */
	typedef void (*StackTrackerReportFn)( const FCallStack& CallStack, uint64 TotalStackCount, FOutputDevice& Ar );
	/** Used to delete a provided userdata pointer e.g. call the correct destructor */
	typedef void (*StackTrackerDeleteUserDataFn)(void*);

	static void DefaultDeleteUserDataFn(void* UserData);

	/** Constructor, initializing all member variables */
	[[nodiscard]] CORE_API FStackTracker(StackTrackerUpdateFn InUpdateFn = nullptr, StackTrackerReportFn InReportFn = nullptr, StackTrackerDeleteUserDataFn InDeleteUserDataFn = nullptr, bool bInIsEnabled = false);

	/**
	 * Captures the current stack and updates stack tracking information.
	 * optionally stores a user data pointer that the tracker will take ownership of and delete upon reset
	 * you must allocate the memory with FMemory::Malloc() or provide a custom delete function
	 * EntriesToIgnore are removed from the top of then stack, then we keep at most StackLen of the remaining entries.
	 */
	CORE_API void CaptureStackTrace(int32 EntriesToIgnore = 2, void* UserData = nullptr, int32 StackLen = MAX_int32, bool bLookupStringsForAliasRemoval = false);

	/**
	 * Dumps capture stack trace summary to the passed in log.
	 */
	CORE_API void DumpStackTraces(int32 StackThreshold, FOutputDevice& Ar, float SampleCountCorrectionFactor = 1.0);

	/** Resets stack tracking. Deletes all user pointers passed in via CaptureStackTrace() */
	CORE_API void ResetTracking();

	/** Toggles tracking. */
	CORE_API void ToggleTracking();
	CORE_API void ToggleTracking(bool bEnable, bool bSilent);

private:

	/** Array of unique callstacks. */
	TArray<FCallStack> CallStacks;
	/** Mapping from callstack CRC to index in callstack array. */
	TMap<uint32,int32> CRCToCallStackIndexMap;
	/** Mapping an address to an arbitrary alias of that address. Only used with bLookupStringsForAliasRemoval.*/
	TMap<uint64,uint64> AliasMap;
	/** Mapping an string to an arbitrary alias of that address. Only used with bLookupStringsForAliasRemoval.*/
	TMap<FString,uint64> StringAliasMap;
	/** Whether we are currently capturing or not, used to avoid re-entrancy. */
	bool bAvoidCapturing;
	/** Whether stack tracking is enabled. */
	bool bIsEnabled;
	/** Frame counter at the time tracking was enabled. */
	uint64 StartFrameCounter;
	/** Frame counter at the time tracking was disabled. */
	uint64 StopFrameCounter;

	/** Used to optionally update the information currently stored with the callstack */
	StackTrackerUpdateFn UpdateFn;
	/** Used to optionally report information based on the current stack */
	StackTrackerReportFn ReportFn;
	/** Used to delete a provided userdata pointer e.g. call the correct destructor */
	StackTrackerDeleteUserDataFn DeleteUserDataFn;
};


