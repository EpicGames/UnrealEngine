// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
// Deprecated in 5.7. Only needed for backward compatibility.
#include "Misc/AlignedElement.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ReverseIterate.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "Delegates/IntegerSequence.h"
#include "Templates/TypeHash.h"

namespace UE::Core::Private
{
	// This is a workaround for a parsing error in MSVC under /persmissive- builds, which would
	// get confused by the fold expression in the constraint in the constructor.
	template <typename InElementType, typename... ArgTypes>
	constexpr bool TCanBeConvertedToFromAll_V = (std::is_convertible_v<ArgTypes, InElementType> && ...);
}

/** An array with a static number of elements. */
template <typename InElementType, uint32 NumElements, uint32 Alignment = uint32(-1)>
class TStaticArray
{
	// Deprecated in 5.7. Friend only needed for backward compatibility.
	template <typename, uint32, uint32> friend class TStaticArray;

public:
	using ElementType = InElementType;

	InElementType Elements[NumElements];

	[[nodiscard]] constexpr TStaticArray() = default;

	// Constructs each element with Args
	template <typename... ArgTypes>
	[[nodiscard]] constexpr explicit TStaticArray(EInPlace, ArgTypes&&... Args)
		: TStaticArray{InPlace, TMakeIntegerSequence<uint32, NumElements>(), [&Args...](uint32) { return InElementType(Args...); }}
	{
	}

	// Directly initializes the array with the provided values.
	template <typename... ArgTypes> requires((sizeof...(ArgTypes) > 0 && sizeof...(ArgTypes) <= NumElements) && UE::Core::Private::TCanBeConvertedToFromAll_V<InElementType, ArgTypes...>)
	[[nodiscard]] constexpr TStaticArray(ArgTypes&&... Args)
		: TStaticArray{PerElement, [&Args] { return InElementType(Forward<ArgTypes>(Args)); }...}
	{
	}

	[[nodiscard]] constexpr TStaticArray(TStaticArray&& Other) = default;
	[[nodiscard]] constexpr TStaticArray(const TStaticArray& Other) = default;
	constexpr TStaticArray& operator=(TStaticArray&& Other) = default;
	constexpr TStaticArray& operator=(const TStaticArray& Other) = default;

	// Accessors.
	[[nodiscard]] constexpr InElementType& operator[](uint32 Index)
	{
		checkSlow(Index < NumElements);
		return Elements[Index];
	}

	[[nodiscard]] constexpr const InElementType& operator[](uint32 Index) const
	{
		checkSlow(Index < NumElements);
		return Elements[Index];
	}

	[[nodiscard]] bool operator==(const TStaticArray&) const = default;

	/**
	 * Returns true if the array is empty and contains no elements. 
	 *
	 * @returns True if the array is empty.
	 * @see Num
	 */
	[[nodiscard]] constexpr bool IsEmpty() const
	{
		return NumElements == 0;
	}

	/** The number of elements in the array. */
	[[nodiscard]] UE_REWRITE constexpr int32 Num() const
	{
		return NumElements;
	}

	/** A pointer to the first element of the array */
	[[nodiscard]] UE_REWRITE constexpr InElementType* GetData()
	{
		return Elements;
	}
	
	[[nodiscard]] UE_REWRITE constexpr const InElementType* GetData() const
	{
		return Elements;
	}

	[[nodiscard]] UE_REWRITE constexpr InElementType*                               begin ()       { return Elements; }
	[[nodiscard]] UE_REWRITE constexpr const InElementType*                         begin () const { return Elements; }
	[[nodiscard]] UE_REWRITE constexpr InElementType*                               end   ()       { return Elements + NumElements; }
	[[nodiscard]] UE_REWRITE constexpr const InElementType*                         end   () const { return Elements + NumElements; }
	[[nodiscard]] UE_REWRITE constexpr TReversePointerIterator<InElementType>       rbegin()       { return TReversePointerIterator<InElementType>(Elements + NumElements); }
	[[nodiscard]] UE_REWRITE constexpr TReversePointerIterator<const InElementType> rbegin() const { return TReversePointerIterator<const InElementType>(Elements + NumElements); }
	[[nodiscard]] UE_REWRITE constexpr TReversePointerIterator<InElementType>       rend  ()       { return TReversePointerIterator<InElementType>(Elements); }
	[[nodiscard]] UE_REWRITE constexpr TReversePointerIterator<const InElementType> rend  () const { return TReversePointerIterator<const InElementType>(Elements); }
	
private:
	template <uint32... Indices, typename ArgGeneratorType>
	constexpr explicit TStaticArray(EInPlace, TIntegerSequence<uint32, Indices...>, ArgGeneratorType Generator)
		: Elements{Generator(Indices)...}
	{
	}

	template <typename... ArgGeneratorTypes>
	constexpr explicit TStaticArray(EPerElement, ArgGeneratorTypes... Generator)
		: Elements{Generator()...}
	{
	}
};

template <typename InElementType, uint32 NumElements, uint32 Alignment> requires(Alignment != uint32(-1))
class TStaticArray<InElementType, NumElements, Alignment>: public TStaticArray<UE::Core::TAlignedElement<InElementType, Alignment>, NumElements>
{
	UE_STATIC_ASSERT_WARN(false, "TStaticArray's alignment parameter has been deprecated in 5.7 - you can use TAlignedElement to wrap InElementType.");
	
	using TStaticArray<UE::Core::TAlignedElement<InElementType, Alignment>, NumElements>::Elements;
 
public:
	using ElementType = InElementType;

	[[nodiscard]] constexpr TStaticArray() = default;

	// Constructs each element with Args
	template <typename... ArgTypes>
	[[nodiscard]] constexpr explicit TStaticArray(EInPlace, ArgTypes&&... Args)
		: TStaticArray<UE::Core::TAlignedElement<InElementType, Alignment>, NumElements>{InPlace, TMakeIntegerSequence<uint32, NumElements>(), [&Args...](uint32) { return InElementType(Args...); }}
	{
	}

	// Directly initializes the array with the provided values.
	template <typename... ArgTypes> requires((sizeof...(ArgTypes) > 0 && sizeof...(ArgTypes) <= NumElements) && UE::Core::Private::TCanBeConvertedToFromAll_V<InElementType, ArgTypes...>)
	[[nodiscard]] constexpr TStaticArray(ArgTypes&&... Args)
		: TStaticArray<UE::Core::TAlignedElement<InElementType, Alignment>, NumElements>{PerElement, [&Args] { return InElementType(Forward<ArgTypes>(Args)); }...}
	{
	}

	[[nodiscard]] constexpr TStaticArray(TStaticArray&& Other) = default;
	[[nodiscard]] constexpr TStaticArray(const TStaticArray& Other) = default;
	constexpr TStaticArray& operator=(TStaticArray&& Other) = default;
	constexpr TStaticArray& operator=(const TStaticArray& Other) = default;
	
	constexpr TStaticArray& operator=(TStaticArray<UE::Core::TAlignedElement<InElementType, Alignment>, NumElements>&& Other)
	{
		TStaticArray<UE::Core::TAlignedElement<InElementType, Alignment>, NumElements>::operator=(MoveTemp(Other));
		return *this;
	}

	constexpr TStaticArray& operator=(const TStaticArray<UE::Core::TAlignedElement<InElementType, Alignment>, NumElements>& Other)
	{
		TStaticArray<UE::Core::TAlignedElement<InElementType, Alignment>, NumElements>::operator=(Other);
		return *this;
	}

	// Accessors.
	[[nodiscard]] inline constexpr InElementType& operator[](uint32 Index)
	{
		checkSlow(Index < NumElements);
		return Elements[Index];
	}

	[[nodiscard]] inline constexpr const InElementType& operator[](uint32 Index) const
	{
		checkSlow(Index < NumElements);
		return Elements[Index];
	}

	template <typename StorageElementType, bool bReverse = false>
	struct FRangedForIterator
	{
		[[nodiscard]] constexpr explicit FRangedForIterator(StorageElementType* InPtr)
			: Ptr(InPtr)
		{
		}

		[[nodiscard]] constexpr std::conditional_t<std::is_const_v<StorageElementType>, const InElementType, InElementType>& operator*() const
		{
			if constexpr (bReverse)
			{
				return *(Ptr - 1);
			}
			else
			{
				return *Ptr;
			}
		}

		constexpr FRangedForIterator& operator++()
		{
			if constexpr (bReverse)
			{
				--Ptr;
			}
			else
			{
				++Ptr;
			}
			return *this;
		}

		[[nodiscard]] constexpr bool operator!=(const FRangedForIterator& B) const
		{
			return Ptr != B.Ptr;
		}

	private:
		StorageElementType* Ptr;
	};

	using RangedForIteratorType             = FRangedForIterator<      UE::Core::TAlignedElement<InElementType, Alignment>>;
	using RangedForConstIteratorType        = FRangedForIterator<const UE::Core::TAlignedElement<InElementType, Alignment>>;
	using RangedForReverseIteratorType      = FRangedForIterator<      UE::Core::TAlignedElement<InElementType, Alignment>, true>;
	using RangedForConstReverseIteratorType = FRangedForIterator<const UE::Core::TAlignedElement<InElementType, Alignment>, true>;

	/** STL-like iterators to enable range-based for loop support. */
	[[nodiscard]] UE_FORCEINLINE_HINT RangedForIteratorType             constexpr begin()        { return RangedForIteratorType(Elements); }
	[[nodiscard]] UE_FORCEINLINE_HINT RangedForConstIteratorType        constexpr begin()  const { return RangedForConstIteratorType(Elements); }
	[[nodiscard]] UE_FORCEINLINE_HINT RangedForIteratorType             constexpr end()          { return RangedForIteratorType(Elements + NumElements); }
	[[nodiscard]] UE_FORCEINLINE_HINT RangedForConstIteratorType        constexpr end()    const { return RangedForConstIteratorType(Elements + NumElements); }
	[[nodiscard]] UE_FORCEINLINE_HINT RangedForReverseIteratorType      constexpr rbegin()       { return RangedForReverseIteratorType(Elements + NumElements); }
	[[nodiscard]] UE_FORCEINLINE_HINT RangedForConstReverseIteratorType constexpr rbegin() const { return RangedForConstReverseIteratorType(Elements + NumElements); }
	[[nodiscard]] UE_FORCEINLINE_HINT RangedForReverseIteratorType      constexpr rend()         { return RangedForReverseIteratorType(Elements); }
	[[nodiscard]] UE_FORCEINLINE_HINT RangedForConstReverseIteratorType constexpr rend()   const { return RangedForConstReverseIteratorType(Elements); }
};

/** Creates a static array filled with the specified value. */
template <typename InElementType, uint32 NumElements>
[[nodiscard]] constexpr TStaticArray<InElementType,NumElements> MakeUniformStaticArray(typename TCallTraits<InElementType>::ParamType InValue)
{
	TStaticArray<InElementType,NumElements> Result;
	for(uint32 ElementIndex = 0;ElementIndex < NumElements;++ElementIndex)
	{
		Result[ElementIndex] = InValue;
	}
	return Result;
}

template <typename ElementType, uint32 NumElements, uint32 Alignment>
struct TIsContiguousContainer<TStaticArray<ElementType, NumElements, Alignment>>
{
	enum { Value = Alignment == uint32(-1) };
};

/** Serializer. */
template <typename ElementType, uint32 NumElements, uint32 Alignment>
FArchive& operator<<(FArchive& Ar,TStaticArray<ElementType, NumElements, Alignment>& StaticArray)
{
	for(uint32 Index = 0;Index < NumElements;++Index)
	{
		Ar << StaticArray[Index];
	}
	return Ar;
}

/** Hash function. */
template <typename ElementType, uint32 NumElements, uint32 Alignment>
[[nodiscard]] uint32 GetTypeHash(const TStaticArray<ElementType, NumElements, Alignment>& Array)
{
	uint32 Hash = 0;
	for (const ElementType& Element : Array)
	{
		Hash = HashCombineFast(Hash, GetTypeHash(Element));
	}
	return Hash;
}
