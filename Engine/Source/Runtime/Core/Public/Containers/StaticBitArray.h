// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"

/** Used to read/write a bit in the static array as a bool. */
template<typename T>
class TStaticBitReference
{
public:

	[[nodiscard]] TStaticBitReference(T& InData,T InMask)
		: Data(InData)
		, Mask(InMask)
	{
	}

	[[nodiscard]] UE_FORCEINLINE_HINT operator bool() const
	{
		 return (Data & Mask) != 0;
	}
	inline void operator=(const bool NewValue)
	{
		if(NewValue)
		{
			Data |= Mask;
		}
		else
		{
			Data &= ~Mask;
		}
	}

private:

	T& Data;
	T Mask;
};


/** Used to read a bit in the static array as a bool. */
template<typename T>
class TConstStaticBitReference
{
public:

	[[nodiscard]] TConstStaticBitReference(const T& InData,T InMask)
		: Data(InData)
		, Mask(InMask)
	{
	}

	[[nodiscard]] UE_FORCEINLINE_HINT operator bool() const
	{
		 return (Data & Mask) != 0;
	}

private:

	const T& Data;
	T Mask;
};


/**
 * A statically sized bit array.
 */
template<uint32 NumBits>
class TStaticBitArray
{
	typedef uint64 WordType;

	struct FBoolType;
	typedef int32* FBoolType::* UnspecifiedBoolType;
	typedef float* FBoolType::* UnspecifiedZeroType;

public:

	/** Minimal initialization constructor */
	[[nodiscard]] UE_FORCEINLINE_HINT TStaticBitArray()
	{
		Clear_();
	}

	/**
	 * Constructor to initialize to a single bit
	 */
	[[nodiscard]] inline TStaticBitArray(bool, uint32 InBitIndex)
	{
		/***********************************************************************

		 If this line fails to compile you are attempting to construct a bit
		 array with an out-of bounds bit index. Follow the compiler errors to
		 the initialization point

		***********************************************************************/
//		static_assert(InBitIndex >= 0 && InBitIndex < NumBits, "Invalid bit value.");

		check((NumBits > 0) ? (InBitIndex<NumBits):1);

		uint32 DestWordIndex = InBitIndex / NumBitsPerWord;
		WordType Word = (WordType)1 << (InBitIndex & (NumBitsPerWord - 1));

		for(int32 WordIndex = 0; WordIndex < NumWords; ++WordIndex)
		{
			Words[WordIndex] = WordIndex == DestWordIndex ? Word : (WordType)0;
		}
	}

	/**
	 * Constructor to initialize from string
	 */
	[[nodiscard]] explicit TStaticBitArray(const FString& Str)
	{
		int32 Length = Str.Len();

		// Trim count to length of bit array
		if(NumBits < Length)
		{
			Length = NumBits;
		}
		Clear_();

		int32 Pos = Length;
		for(int32 Index = 0; Index < Length; ++Index)
		{
			const TCHAR ch = Str[--Pos];
			if(ch == TEXT('1'))
			{
				operator[](Index) = true;
			}
			else if(ch != TEXT('0'))
			{
				ErrorInvalid_();
			}
		}
	}

	[[nodiscard]] inline bool HasAnyBitsSet() const
	{
		WordType And = 0;
		for(int32 Index = 0; Index < NumWords; ++Index)
		{
			And |= Words[Index];
		}

		return And != 0;
	}

	// Explicit conversion to bool
	[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
	{
		return this->HasAnyBitsSet();
	}

	// Accessors.
	[[nodiscard]] UE_FORCEINLINE_HINT static int32 Num()
	{
		return NumBits;
	}
	[[nodiscard]] inline TStaticBitReference<WordType> operator[](int32 Index)
	{
		check(Index>=0 && Index<NumBits);
		return TStaticBitReference<WordType>(
			Words[Index / NumBitsPerWord],
			(WordType)1 << (Index & (NumBitsPerWord - 1))
			);
	}
	[[nodiscard]] inline const TConstStaticBitReference<WordType> operator[](int32 Index) const
	{
		check(Index>=0 && Index<NumBits);
		return TConstStaticBitReference<WordType>(
			Words[Index / NumBitsPerWord],
			(WordType)1 << (Index & (NumBitsPerWord - 1))
			);
	}

	// Modifiers.
	inline TStaticBitArray& operator|=(const TStaticBitArray& Other)
	{
		for(int32 Index = 0; Index < NumWords; ++Index)
		{
			Words[Index] |= Other.Words[Index];
		}
		return *this;
	}
	inline TStaticBitArray& operator&=(const TStaticBitArray& Other)
	{
		for(int32 Index = 0; Index < NumWords; ++Index)
		{
			Words[Index] &= Other.Words[Index];
		}
		return *this;
	}
	inline TStaticBitArray& operator^=(const TStaticBitArray& Other )
	{
		for(int32 Index = 0; Index < NumWords; ++Index)
		{
			Words[Index] ^= Other.Words[Index];
		}
		return *this;
	}
	[[nodiscard]] friend inline TStaticBitArray<NumBits> operator~(const TStaticBitArray<NumBits>& A)
	{
		TStaticBitArray Result;
		for(int32 Index = 0; Index < NumWords; ++Index)
		{
			Result.Words[Index] = ~A.Words[Index];
		}
		Result.Trim_();
		return Result;
	}
	[[nodiscard]] friend inline TStaticBitArray<NumBits> operator|(const TStaticBitArray<NumBits>& A, const TStaticBitArray<NumBits>& B)
	{
		// is not calling |= because doing it in here has less LoadHitStores and is therefore faster.
		TStaticBitArray Results(0);

		const WordType* RESTRICT APtr = (const WordType* RESTRICT)A.Words;
		const WordType* RESTRICT BPtr = (const WordType* RESTRICT)B.Words;
		WordType* RESTRICT ResultsPtr = (WordType* RESTRICT)Results.Words;
		for(int32 Index = 0; Index < NumWords; ++Index)
		{
			ResultsPtr[Index] = APtr[Index] | BPtr[Index];
		}

		return Results;
	}
	[[nodiscard]] friend inline TStaticBitArray<NumBits> operator&(const TStaticBitArray<NumBits>& A, const TStaticBitArray<NumBits>& B)
	{
		// is not calling &= because doing it in here has less LoadHitStores and is therefore faster.
		TStaticBitArray Results(0);

		const WordType* RESTRICT APtr = (const WordType* RESTRICT)A.Words;
		const WordType* RESTRICT BPtr = (const WordType* RESTRICT)B.Words;
		WordType* RESTRICT ResultsPtr = (WordType* RESTRICT)Results.Words;
		for(int32 Index = 0; Index < NumWords; ++Index)
		{
			ResultsPtr[Index] = APtr[Index] & BPtr[Index];
		}

		return Results;
	}
	[[nodiscard]] friend inline TStaticBitArray<NumBits> operator^(const TStaticBitArray<NumBits>& A, const TStaticBitArray<NumBits>& B)
	{
		TStaticBitArray Results(A);
		Results ^= B;
		return Results;
	}
	[[nodiscard]] inline bool operator==(const TStaticBitArray<NumBits>& B) const
	{
		for(int32 Index = 0; Index < NumWords; ++Index)
		{
			if(Words[Index] != B.Words[Index])
			{
				return false;
			}
		}
		return true;
	}
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TStaticBitArray<NumBits>& B) const
	{
		return !(*this == B);
	}
	
	/**
	 * Finds the first clear bit in the array and returns the bit index.
	 * If there isn't one, INDEX_NONE is returned.
	 */
	[[nodiscard]] int32 FindFirstClearBit() const
	{
		const int32 LocalNumBits = NumBits;

		int32 WordIndex = 0;
		// Iterate over the array until we see a word with a unset bit.
		while (WordIndex < NumWords && Words[WordIndex] == WordType(-1))
		{
			++WordIndex;
		}

		if (WordIndex < NumWords)
		{
			// Flip the bits, then we only need to find the first one bit -- easy.
			const WordType Bits = ~(Words[WordIndex]);
			UE_ASSUME(Bits != 0);

			const int32 LowestBitIndex = (int32)FMath::CountTrailingZeros64(Bits) + (WordIndex << NumBitsPerWordLog2);
			if (LowestBitIndex < LocalNumBits)
			{
				return LowestBitIndex;
			}
		}

		return INDEX_NONE;
	}

	/**
	 * Finds the first set bit in the array and returns it's index.
	 * If there isn't one, INDEX_NONE is returned.
	 */
	[[nodiscard]] int32 FindFirstSetBit() const
	{
		const int32 LocalNumBits = NumBits;

		int32 WordIndex = 0;
		// Iterate over the array until we see a word with a set bit.
		while (WordIndex < NumWords && Words[WordIndex] == WordType(0))
		{
			++WordIndex;
		}

		if (WordIndex < NumWords)
		{
			const WordType Bits = Words[WordIndex];
			UE_ASSUME(Bits != 0);

			const int32 LowestBitIndex = (int32)FMath::CountTrailingZeros64(Bits) + (WordIndex << NumBitsPerWordLog2);
			if (LowestBitIndex < LocalNumBits)
			{
				return LowestBitIndex;
			}
		}

		return INDEX_NONE;
	}

	/**
	 * Converts the bitarray to a string representing the binary representation of the array
	 */
	[[nodiscard]] FString ToString() const
	{
		FString Str;
		Str.Empty(NumBits);

		for(int32 Index = NumBits - 1; Index >= 0; --Index)
		{
			Str += operator[](Index) ? TEXT('1') : TEXT('0');
		}

		return Str;
	}

	void Serialize(FArchive& Ar)
	{
		uint32 ArchivedNumWords = NumWords;
		Ar << ArchivedNumWords;

		if (Ar.IsLoading())
		{
			FMemory::Memset(Words, 0, sizeof(Words));
			ArchivedNumWords = FMath::Min(NumWords, ArchivedNumWords);
		}

		Ar.Serialize(Words, ArchivedNumWords * sizeof(Words[0]));
	}
	
	/**
	 * Fills the entire bit array with zeros or ones.
	 */
	void Fill(bool Value)
	{
		FMemory::Memset(Words, Value ? 0xFF : 0x00, sizeof(Words));
	}

	static constexpr uint32 NumOfBits = NumBits;

private:
	static constexpr uint32 NumBitsPerWord = sizeof(WordType) * 8;
	static constexpr uint32 NumBitsPerWordLog2 = 6;
	static_assert(NumBitsPerWord == (1u << NumBitsPerWordLog2), "Update NumBitsPerWordLog2 to reflect WordType");
	static constexpr uint32 NumWords = ((NumBits + NumBitsPerWord - 1) & ~(NumBitsPerWord - 1)) / NumBitsPerWord;
	WordType Words[NumWords];

	// Helper class for bool conversion
	struct FBoolType
	{
		int32* Valid;
	};

	/**
	 * Resets the bit array to a 0 value
	 */
	inline void Clear_()
	{
		for(int32 Index = 0; Index < NumWords; ++Index)
		{
			Words[Index] = 0;
		}
	}

	/**
	 * Clears any trailing bits in the last word
	 */
	void Trim_()
	{
		constexpr uint32 NumOverflowBits = NumBits % NumBitsPerWord;
		if constexpr (NumOverflowBits != 0)
		{
			Words[NumWords-1] &= (WordType(1) << NumOverflowBits) - 1;
		}
	}

	/**
	 * Reports an invalid string element in the bitset conversion
	 */
	void ErrorInvalid_() const
	{
		LowLevelFatalError(TEXT("invalid TStaticBitArray<NumBits> character"));
	}
};

/**
	* Serializer.
	*/
template<uint32 NumBits>
FArchive& operator<<(FArchive& Ar, TStaticBitArray<NumBits>& BitArray)
{
	BitArray.Serialize(Ar);
	return Ar;
}
