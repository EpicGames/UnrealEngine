// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "Templates/UnrealTemplate.h"
#include "Misc/AssertionMacros.h"
#include "Containers/ArrayView.h"
#include "HAL/PlatformString.h" // for INT64_FMT

#include <type_traits>

/*
* TStridedView is similar to TArrayView, but allows flexible byte stride between elements.
* Stride must be positive and a multiple of the element alignment.
* Stride can be zero to duplicate the same element over the whole range.
* 
* Example usage:
* 
* struct FMyStruct
* {
*     uint32  SomeData;
*     FVector Position;
* };
*
* FVector ComputeMean(TStridedView<const FVector> Positions)
* {
*     return Algo::Accumulate(Positions, FVector(0.0f)) / Positions.Num();
* }
*
* FVector ComputeMeanPosition(TArrayView<const FMyStruct> Structs)
* {
*     return ComputeMean(MakeStridedView(Structs, &FMyStruct::Position))
* }
* 
* See StridedViewTest.cpp for more examples.
* 
*/

/** Pointer with extent and a stride, similar to TArrayView. Designed to allow functions to take pointers to arbitrarily structured data. */
template<typename InElementType, typename InSizeType>
class TStridedView
{
public:

	using ElementType = InElementType;
	using SizeType = InSizeType;

	static_assert(std::is_signed_v<SizeType>, "TStridedView only supports signed index types");

	[[nodiscard]] TStridedView() = default;

	template <
		typename OtherElementType
		UE_REQUIRES(std::is_convertible_v<OtherElementType**, ElementType* const*>)
	>
	[[nodiscard]] inline TStridedView(SizeType InBytesBetweenElements, OtherElementType* InFirstElementPtr, SizeType InNumElements)
		: FirstElementPtr(InFirstElementPtr)
		, BytesBetweenElements(InBytesBetweenElements)
		, NumElements(InNumElements)
	{
		check(NumElements >= 0);
		check(BytesBetweenElements >= 0); // NOTE: Zero stride is valid to allow duplicating a single element.
		check(BytesBetweenElements % alignof(ElementType) == 0);
	}

	template <
		typename OtherElementType
		UE_REQUIRES(std::is_convertible_v<OtherElementType**, ElementType* const*>)
	>
	[[nodiscard]] inline TStridedView(const TStridedView<OtherElementType, SizeType>& Other)
		: FirstElementPtr(nullptr)
		, BytesBetweenElements(Other.GetStride())
		, NumElements(Other.Num())
	{
		if (Other.Num())
		{
			FirstElementPtr = &Other[0];
		}
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValidIndex(SizeType Index) const
	{
		return (Index >= 0) && (Index < NumElements);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return NumElements == 0;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT SizeType Num() const
	{
		return NumElements;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT SizeType GetStride() const
	{
		return BytesBetweenElements;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& GetUnsafe(SizeType Index) const
	{
		return *GetElementPtrUnsafe(Index);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& operator[](SizeType Index) const
	{
		return *GetElementPtr(Index);
	}

	struct FIterator
	{
		const TStridedView* Owner;
		SizeType Index;

		UE_FORCEINLINE_HINT FIterator& operator++()
		{
			++Index;
			return *this;
		}

		UE_FORCEINLINE_HINT ElementType& operator*()
		{
			return *Owner->GetElementPtrUnsafe(Index);
		}

		[[nodiscard]] inline bool operator==(const FIterator& Other) const
		{
			return Owner == Other.Owner
				&& Index == Other.Index;
		}

		[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const FIterator& Other) const
		{
			return !(*this == Other);
		}
	};

	/** Ranged iteration support. DO NOT USE DIRECTLY. */

	[[nodiscard]] UE_FORCEINLINE_HINT FIterator begin() const { return FIterator{ this, 0 }; }
	[[nodiscard]] UE_FORCEINLINE_HINT FIterator end() const { return FIterator{ this, Num() }; }

private:

	UE_FORCEINLINE_HINT void RangeCheck(SizeType Index) const
	{
		checkf((Index >= 0) & (Index < NumElements), TEXT("Array index out of bounds: %" INT64_FMT " from an array of size %" INT64_FMT), int64(Index), int64(NumElements))
	}

	[[nodiscard]] inline ElementType* GetElementPtrUnsafe(SizeType Index) const
	{
		using ByteType = typename std::conditional_t<std::is_const_v<ElementType>, const uint8, uint8>;

		ByteType* AsBytes = reinterpret_cast<ByteType*>(FirstElementPtr);
		ElementType* AsElement = reinterpret_cast<ElementType*>(AsBytes + uint64(Index) * uint64(BytesBetweenElements));

		return AsElement;
	}

	[[nodiscard]] inline ElementType* GetElementPtr(SizeType Index) const
	{
		RangeCheck(Index);
		return GetElementPtrUnsafe(Index);
	}

private:

	ElementType* FirstElementPtr = nullptr;
	SizeType BytesBetweenElements = 0;
	SizeType NumElements = 0;
};

template <typename ElementType>
[[nodiscard]] TStridedView<ElementType> MakeStridedView(int32 BytesBetweenElements, ElementType* FirstElement, int32 Count)
{
	return TStridedView<ElementType>(BytesBetweenElements, FirstElement, Count);
}

template <typename ElementType>
[[nodiscard]] TConstStridedView<ElementType> MakeConstStridedView(int32 BytesBetweenElements, const ElementType* FirstElement, int32 Count)
{
	return TConstStridedView<ElementType>(BytesBetweenElements, FirstElement, Count);
}

template <typename BaseStructureType, typename DerivedStructureType>
[[nodiscard]] TStridedView<BaseStructureType> MakeStridedViewOfBase(TArrayView<DerivedStructureType> StructuredView)
{
	static_assert(std::is_base_of_v<BaseStructureType, DerivedStructureType>, "Expecting derived structure type");
	return MakeStridedView<BaseStructureType>((int32)sizeof(DerivedStructureType), GetData(StructuredView), (int32)GetNum(StructuredView));
}

template <typename BaseStructureType, typename DerivedStructureType>
[[nodiscard]] TConstStridedView<BaseStructureType> MakeConstStridedViewOfBase(TConstArrayView<DerivedStructureType> StructuredView)
{
	static_assert(std::is_base_of_v<BaseStructureType, DerivedStructureType>, "Expecting derived structure type");
	return MakeConstStridedView<BaseStructureType>((int32)sizeof(DerivedStructureType), GetData(StructuredView), (int32)GetNum(StructuredView));
}

template <typename StructureType>
[[nodiscard]] TStridedView<StructureType> MakeStridedView(TArrayView<StructureType> StructuredView)
{
	return MakeStridedView((int32)sizeof(StructureType), GetData(StructuredView), (int32)GetNum(StructuredView));
}

template <typename StructureType>
[[nodiscard]] TConstStridedView<StructureType> MakeConstStridedView(TConstArrayView<StructureType> StructuredView)
{
	return MakeConstStridedView((int32)sizeof(StructureType), GetData(StructuredView), (int32)GetNum(StructuredView));
}

template <
	typename StructuredRangeType,
	typename ElementType,
	typename StructureType,
	decltype(GetData(std::declval<StructuredRangeType>()))* = nullptr,
	decltype(GetNum(std::declval<StructuredRangeType>()))* = nullptr
>
[[nodiscard]] TStridedView<ElementType> MakeStridedView(StructuredRangeType&& StructuredRange, ElementType StructureType::* Member)
{
	return TStridedView<ElementType>((int32)sizeof(*GetData(StructuredRange)), &(GetData(StructuredRange)->*Member), (int32)GetNum(StructuredRange));
}

template <
	typename StructuredRangeType,
	typename ElementType,
	typename StructureType,
	decltype(GetData(std::declval<StructuredRangeType>()))* = nullptr,
	decltype(GetNum(std::declval<StructuredRangeType>()))* = nullptr
>
[[nodiscard]] TConstStridedView<ElementType> MakeConstStridedView(StructuredRangeType&& StructuredRange, const ElementType StructureType::* Member)
{
	return TConstStridedView<ElementType>((int32)sizeof(*GetData(StructuredRange)), &(GetData(StructuredRange)->*Member), (int32)GetNum(StructuredRange));
}

template <typename StructuredRangeType>
[[nodiscard]] auto MakeStridedView(StructuredRangeType&& StructuredRange) -> TStridedView<std::remove_reference_t<decltype(*GetData(StructuredRange))>>
{
	auto StructuredView = MakeArrayView(Forward<StructuredRangeType>(StructuredRange));
	return MakeStridedView((int32)sizeof(*GetData(StructuredRange)), GetData(StructuredView), (int32)GetNum(StructuredView));
}

template <typename StructuredRangeType>
[[nodiscard]] auto MakeConstStridedView(StructuredRangeType&& StructuredRange) -> TConstStridedView<std::remove_reference_t<decltype(*GetData(StructuredRange))>>
{
	auto StructuredView = MakeArrayView(Forward<StructuredRangeType>(StructuredRange));
	return MakeConstStridedView((int32)sizeof(*GetData(StructuredRange)), GetData(StructuredView), (int32)GetNum(StructuredView));
}
