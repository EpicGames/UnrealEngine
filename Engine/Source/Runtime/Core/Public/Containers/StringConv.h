// Copyright Epic Games, Inc. All Rights Reserved.

// This file contains the classes used when converting strings between
// standards (ANSI, UNICODE, etc.)

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Array.h"
#include "Misc/CString.h"
#include "Templates/IsArray.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "Traits/ElementType.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "Traits/IsContiguousContainer.h"
#include <type_traits>

#define DEFAULT_STRING_CONVERSION_SIZE 128u

template <typename From, typename To>
class TStringConvert
{
public:
	typedef From FromType;
	typedef To   ToType;

	template <
		typename CharType
		UE_REQUIRES(TIsCharEncodingCompatibleWith_V<CharType, FromType>)
	>
	inline static void Convert(To* Dest, int32 DestLen, const CharType* Source, int32 SourceLen)
	{
		To* Result = FPlatformString::Convert(Dest, DestLen, (const FromType*)Source, SourceLen);
		check(Result);
	}

	template <
		typename CharType
		UE_REQUIRES(TIsCharEncodingCompatibleWith_V<CharType, FromType>)
	>
	[[nodiscard]] static int32 ConvertedLength(const CharType* Source, int32 SourceLen)
	{
		return FPlatformString::ConvertedLength<To>((const FromType*)Source, SourceLen);
	}
};

namespace StringConv
{
	/** Is the provided Codepoint within the range of valid codepoints? */
	[[nodiscard]] inline bool IsValidCodepoint(const uint32 Codepoint)
	{
		if ((Codepoint > 0x10FFFF) ||						// No Unicode codepoints above 10FFFFh, (for now!)
			(Codepoint == 0xFFFE) || (Codepoint == 0xFFFF)) // illegal values.
		{
			return false;
		}
		return true;
	}

	/** Is the provided Codepoint within the range of the high-surrogates? */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsHighSurrogate(const uint32 Codepoint)
	{
		return Codepoint >= HIGH_SURROGATE_START_CODEPOINT && Codepoint <= HIGH_SURROGATE_END_CODEPOINT;
	}

	/** Is the provided Codepoint within the range of the low-surrogates? */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsLowSurrogate(const uint32 Codepoint)
	{
		return Codepoint >= LOW_SURROGATE_START_CODEPOINT && Codepoint <= LOW_SURROGATE_END_CODEPOINT;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT uint32 EncodeSurrogate(const uint16 HighSurrogate, const uint16 LowSurrogate)
	{
		return ((HighSurrogate - HIGH_SURROGATE_START_CODEPOINT) << 10) + (LowSurrogate - LOW_SURROGATE_START_CODEPOINT) + 0x10000;
	}

	inline void DecodeSurrogate(const uint32 Codepoint, uint16& OutHighSurrogate, uint16& OutLowSurrogate)
	{
		const uint32 TmpCodepoint = Codepoint - 0x10000;
		OutHighSurrogate = (uint16)((TmpCodepoint >> 10) + HIGH_SURROGATE_START_CODEPOINT);
		OutLowSurrogate = (TmpCodepoint & 0x3FF) + LOW_SURROGATE_START_CODEPOINT;
	}

	/** Is the provided Codepoint outside of the range of the basic multilingual plane, but within the valid range of UTF8/16? */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsEncodedSurrogate(const uint32 Codepoint)
	{
		return Codepoint >= ENCODED_SURROGATE_START_CODEPOINT && Codepoint <= ENCODED_SURROGATE_END_CODEPOINT;
	}

	/** Inline combine any UTF-16 surrogate pairs in the given null-terminated character buffer, returning the updated length */
	template<typename CharType>
	inline int32 InlineCombineSurrogates_Buffer(CharType* StrBuffer, int32 StrLen)
	{
		static_assert(sizeof(CharType) == 4, "CharType must be 4-bytes!");

		for (int32 Index = 0; Index < StrLen; ++Index)
		{
			CharType Codepoint = StrBuffer[Index];

			// Check if this character is a high-surrogate
			if (StringConv::IsHighSurrogate(Codepoint))
			{
				// Ensure our string has enough characters to read from
				if ((Index + 1) >= StrLen)
				{
					// Unpaired surrogate - replace with bogus char
					StrBuffer[Index] = UNICODE_BOGUS_CHAR_CODEPOINT;
					break;
				}

				const uint32 HighCodepoint = Codepoint;
				Codepoint = StrBuffer[Index + 1];

				// If our High Surrogate is set, check if this character is the matching low-surrogate
				if (StringConv::IsLowSurrogate(Codepoint))
				{
					const uint32 LowCodepoint = Codepoint;

					// Combine our high and low surrogates together to a single Unicode codepoint
					Codepoint = StringConv::EncodeSurrogate(static_cast<uint16>(HighCodepoint), static_cast<uint16>(LowCodepoint));

					StrBuffer[Index] = Codepoint;
					{
						const int32 ArrayNum = StrLen + 1;
						const int32 IndexToRemove = Index + 1;
						const int32 NumToMove = ArrayNum - IndexToRemove - 1;
						if (NumToMove > 0)
						{
							FMemory::Memmove(StrBuffer + IndexToRemove, StrBuffer + IndexToRemove + 1, NumToMove * sizeof(CharType));
						}
					}
					--StrLen;
					continue;
				}

				// Unpaired surrogate - replace with bogus char
				StrBuffer[Index] = UNICODE_BOGUS_CHAR_CODEPOINT;
			}
			else if (StringConv::IsLowSurrogate(Codepoint))
			{
				// Unpaired surrogate - replace with bogus char
				StrBuffer[Index] = UNICODE_BOGUS_CHAR_CODEPOINT;
			}
		}

		return StrLen;
	}

	/** Inline combine any UTF-16 surrogate pairs in the given null-terminated TCHAR array */
	template<typename AllocatorType>
	UE_DEPRECATED(5.8, "Remove calls to this function because it does nothing.")
	inline void InlineCombineSurrogates_Array(TArray<TCHAR, AllocatorType>& StrBuffer)
	{
	}

	struct FUnused;

	// Helper functions for getting the conversion type from a converter
	template <typename ConverterType>
	typename ConverterType::LegacyFromType GetLegacyFromType(typename ConverterType::LegacyFromType*);

	template <typename ConverterType>
	FUnused GetLegacyFromType(...);
}

namespace UE::Core::Private
{
	/**
	 * This is a basic object which counts how many times it has been incremented
	 */
	struct FCountingOutputIterator
	{
		[[nodiscard]] FCountingOutputIterator()
			: Counter(0)
		{
		}

		[[nodiscard]] const FCountingOutputIterator& operator*() const
		{
			return *this;
		}

		const FCountingOutputIterator& operator++()
		{
			++Counter;
			return *this;
		}
		const FCountingOutputIterator& operator++(int)
		{
			++Counter;
			return *this;
		}
		const FCountingOutputIterator& operator+=(const int32 Amount)
		{
			Counter += Amount;
			return *this;
		}

		template <typename T>
		T operator=(T Val) const
		{
			return Val;
		}

		[[nodiscard]] friend int32 operator-(FCountingOutputIterator Lhs, FCountingOutputIterator Rhs)
		{
			return Lhs.Counter - Rhs.Counter;
		}

		[[nodiscard]] int32 GetCount() const
		{
			return Counter;
		}

	private:
		int32 Counter;
	};

	// This should be replaced with Platform stuff when FPlatformString starts to know about UTF-8.
	class FTCHARToUTF8_Convert
	{
		// Leave this until we convert all the call sites and can change FromType to this.
		using IntendedToType = UTF8CHAR;
	
	public:
		typedef TCHAR    FromType;
		typedef ANSICHAR ToType;
	
		/**
		 * Convert Codepoint into UTF-8 characters.
		 *
		 * @param Codepoint Codepoint to expand into UTF-8 bytes
		 * @param OutputIterator Output iterator to write UTF-8 bytes into
		 * @param OutputIteratorByteSizeRemaining Maximum number of ANSI characters that can be written to OutputIterator
		 * @return Number of characters written for Codepoint
		 */
		template <typename BufferType>
		[[nodiscard]] static int32 Utf8FromCodepoint(uint32 Codepoint, BufferType OutputIterator, uint32 OutputIteratorByteSizeRemaining)
		{
			// Ensure we have at least one character in size to write
			if (OutputIteratorByteSizeRemaining < sizeof(ToType))
			{
				return 0;
			}
	
			const BufferType OutputIteratorStartPosition = OutputIterator;
	
			if (!StringConv::IsValidCodepoint(Codepoint))
			{
				Codepoint = UNICODE_BOGUS_CHAR_CODEPOINT;
			}
			else if (StringConv::IsHighSurrogate(Codepoint) || StringConv::IsLowSurrogate(Codepoint)) // UTF-8 Characters are not allowed to encode codepoints in the surrogate pair range
			{
				Codepoint = UNICODE_BOGUS_CHAR_CODEPOINT;
			}
	
			// Do the encoding...
			if (Codepoint < 0x80)
			{
				*(OutputIterator++) = (ToType)Codepoint;
			}
			else if (Codepoint < 0x800)
			{
				if (OutputIteratorByteSizeRemaining >= 2)
				{
					*(OutputIterator++) = (ToType)((Codepoint >> 6)         | 128 | 64);
					*(OutputIterator++) = (ToType)((Codepoint       & 0x3F) | 128);
				}
			}
			else if (Codepoint < 0x10000)
			{
				if (OutputIteratorByteSizeRemaining >= 3)
				{
					*(OutputIterator++) = (ToType) ((Codepoint >> 12)        | 128 | 64 | 32);
					*(OutputIterator++) = (ToType)(((Codepoint >> 6) & 0x3F) | 128);
					*(OutputIterator++) = (ToType) ((Codepoint       & 0x3F) | 128);
				}
			}
			else
			{
				if (OutputIteratorByteSizeRemaining >= 4)
				{
					*(OutputIterator++) = (ToType) ((Codepoint >> 18)         | 128 | 64 | 32 | 16);
					*(OutputIterator++) = (ToType)(((Codepoint >> 12) & 0x3F) | 128);
					*(OutputIterator++) = (ToType)(((Codepoint >> 6 ) & 0x3F) | 128);
					*(OutputIterator++) = (ToType) ((Codepoint        & 0x3F) | 128);
				}
			}
	
			return UE_PTRDIFF_TO_INT32(OutputIterator - OutputIteratorStartPosition);
		}
	
		/**
		 * Converts a Source string into UTF8 and stores it in Dest.
		 *
		 * @param Dest      The destination output iterator. Usually ANSICHAR*, but you can supply your own output iterator.
		 *                  One can determine the number of characters written by checking the offset of Dest when the function returns.
		 * @param DestLen   The length of the destination buffer.
		 * @param Source    The source string to convert.
		 * @param SourceLen The length of the source string.
		 * @return          The number of bytes written to Dest, up to DestLen, or -1 if the entire Source string could did not fit in DestLen bytes.
		 */
		template <
			typename SrcBufferType
			UE_REQUIRES(TIsCharEncodingCompatibleWith_V<SrcBufferType, FromType>)
		>
		static inline int32 Convert(IntendedToType* Dest, int32 DestLen, const SrcBufferType* Source, int32 SourceLen)
		{
			IntendedToType* Result = FPlatformString::Convert(Dest, DestLen, (const FromType*)Source, SourceLen);
			if (!Result)
			{
				return -1;
			}
	
			return (int32)(Result - Dest);
		}
		template <
			typename SrcBufferType
			UE_REQUIRES(TIsCharEncodingCompatibleWith_V<SrcBufferType, FromType>)
		>
		static UE_FORCEINLINE_HINT int32 Convert(ToType* Dest, int32 DestLen, const SrcBufferType* Source, int32 SourceLen)
		{
			// This helper function is only for backwards compatibility and won't be necessary once IntendedToType becomes the ToType
			return FTCHARToUTF8_Convert::Convert((IntendedToType*)Dest, DestLen, Source, SourceLen);
		}
	
		/**
		 * Determines the length of the converted string.
		 *
		 * @return The length of the string in UTF-8 code units.
		 */
		template <
			typename SrcBufferType
			UE_REQUIRES(TIsCharEncodingCompatibleWith_V<SrcBufferType, FromType>)
		>
		[[nodiscard]] static UE_FORCEINLINE_HINT int32 ConvertedLength(const SrcBufferType* Source, int32 SourceLen)
		{
			return FPlatformString::ConvertedLength<IntendedToType>((const FromType*)Source, SourceLen);
		}
	};

	enum class ENullTerminatedString
	{
		No  = 0,
		Yes = 1
	};

	// This tag type is only intended to be used by StrCast to construct existing classes in direct ways.
	// It is defined is in this Private namespace to indicate that the constructors that use it are not to be used by end-users.
	struct FFromStrCast
	{
		explicit FFromStrCast() = default;
	};
}

using FTCHARToUTF8_Convert /*UE_DEPRECATED(5.1, "FTCHARToUTF8_Convert has been deprecated in favor of FPlatformString::Convert and StringCast")*/ = UE::Core::Private::FTCHARToUTF8_Convert;

class FUTF8ToTCHAR_Convert
{
public:
	typedef ANSICHAR LegacyFromType;
	typedef UTF8CHAR FromType;
	typedef TCHAR    ToType;

	/**
	 * Converts the UTF-8 string to UTF-16 or UTF-32.
	 *
	 * @param Dest      The destination buffer of the converted string.
	 * @param DestLen   The length of the destination buffer.
	 * @param Source    The source string to convert.
	 * @param SourceLen The length of the source string.
	 */
	template <
		typename SrcBufferType
		UE_REQUIRES(TIsCharEncodingCompatibleWith_V<SrcBufferType, FromType>)
	>
	static UE_FORCEINLINE_HINT void Convert(ToType* Dest, const int32 DestLen, const SrcBufferType* Source, const int32 SourceLen)
	{
		FPlatformString::Convert(Dest, DestLen, (const FromType*)Source, SourceLen);
	}
	static UE_FORCEINLINE_HINT void Convert(ToType* Dest, const int32 DestLen, const LegacyFromType* Source, const int32 SourceLen)
	{
		FPlatformString::Convert(Dest, DestLen, (const FromType*)Source, SourceLen);
	}

	/**
	 * Determines the length of the converted string.
	 *
	 * @param Source string to read and determine amount of TCHARs to represent
	 * @param SourceLen Length of source string; we will not read past this amount, even if the characters tell us to
	 * @return The length of the string in UTF-16 or UTF-32 characters.
	 */
	template <
		typename SrcBufferType
		UE_REQUIRES(TIsCharEncodingCompatibleWith_V<SrcBufferType, FromType>)
	>
	[[nodiscard]] static int32 ConvertedLength(const SrcBufferType* Source, const int32 SourceLen)
	{
		return FPlatformString::ConvertedLength<ToType>((const FromType*)Source, SourceLen);
	}
	[[nodiscard]] static int32 ConvertedLength(const LegacyFromType* Source, const int32 SourceLen)
	{
		return FPlatformString::ConvertedLength<ToType>((const FromType*)Source, SourceLen);
	}
};

template<typename InFromType, typename InToType>
class TUTF32ToUTF16_Convert
{
	static_assert(sizeof(InFromType) == 4, "FromType must be 4 bytes!");
	static_assert(sizeof(InToType) == 2, "ToType must be 2 bytes!");

public:
	typedef InFromType FromType;
	typedef InToType   ToType;

	/**
	 * Convert Codepoint into UTF-16 characters.
	 *
	 * @param Codepoint Codepoint to expand into UTF-16 code units
	 * @param OutputIterator Output iterator to write UTF-16 code units into
	 * @param OutputIteratorNumRemaining Maximum number of UTF-16 code units that can be written to OutputIterator
	 * @return Number of characters written for Codepoint
	 */
	template <typename BufferType>
	[[nodiscard]] static int32 Utf16FromCodepoint(uint32 Codepoint, BufferType OutputIterator, uint32 OutputIteratorNumRemaining)
	{
		// Ensure we have at least one character in size to write
		if (OutputIteratorNumRemaining < 1)
		{
			return 0;
		}

		const BufferType OutputIteratorStartPosition = OutputIterator;

		if (!StringConv::IsValidCodepoint(Codepoint))
		{
			Codepoint = UNICODE_BOGUS_CHAR_CODEPOINT;
		}
		else if (StringConv::IsHighSurrogate(Codepoint) || StringConv::IsLowSurrogate(Codepoint)) // Surrogate pairs range should not be present in UTF-32
		{
			Codepoint = UNICODE_BOGUS_CHAR_CODEPOINT;
		}

		// We want to write out two chars
		if (StringConv::IsEncodedSurrogate(Codepoint))
		{
			// We need two characters to write the surrogate pair
			if (OutputIteratorNumRemaining >= 2)
			{
				uint16 HighSurrogate = 0;
				uint16 LowSurrogate = 0;
				StringConv::DecodeSurrogate(Codepoint, HighSurrogate, LowSurrogate);

				*(OutputIterator++) = (ToType)HighSurrogate;
				*(OutputIterator++) = (ToType)LowSurrogate;
			}
		}
		else if (Codepoint > ENCODED_SURROGATE_END_CODEPOINT)
		{
			// Ignore values higher than the supplementary plane range
			*(OutputIterator++) = UNICODE_BOGUS_CHAR_CODEPOINT;
		}
		else
		{
			// Normal codepoint
			*(OutputIterator++) = (ToType)Codepoint;
		}

		return UE_PTRDIFF_TO_INT32(OutputIterator - OutputIteratorStartPosition);
	}

	/**
	 * Converts the string to the desired format.
	 *
	 * @param Dest      The destination buffer of the converted string.
	 * @param DestLen   The length of the destination buffer.
	 * @param Source    The source string to convert.
	 * @param SourceLen The length of the source string.
	 */
	template <
		typename SrcBufferType
		UE_REQUIRES(TIsCharEncodingCompatibleWith_V<SrcBufferType, FromType>)
	>
	static UE_FORCEINLINE_HINT void Convert(ToType* Dest, int32 DestLen, const SrcBufferType* Source, int32 SourceLen)
	{
		Convert_Impl(Dest, DestLen, (const FromType*)Source, SourceLen);
	}

	/**
	 * Determines the length of the converted string.
	 *
	 * @return The length of the string in UTF-16 code units.
	 */
	template <
		typename SrcBufferType
		UE_REQUIRES(TIsCharEncodingCompatibleWith_V<SrcBufferType, FromType>)
	>
	[[nodiscard]] static inline int32 ConvertedLength(const SrcBufferType* Source, int32 SourceLen)
	{
		UE::Core::Private::FCountingOutputIterator Dest;
		const int32 DestLen = SourceLen * 2;
		Convert_Impl(Dest, DestLen, (const FromType*)Source, SourceLen);

		return Dest.GetCount();
	}

private:
	template <typename DestBufferType>
	static void Convert_Impl(DestBufferType& Dest, int32 DestLen, const FromType* Source, const int32 SourceLen)
	{
		for (int32 i = 0; i < SourceLen; ++i)
		{
			uint32 Codepoint = static_cast<uint32>(Source[i]);

			if (!WriteCodepointToBuffer(Codepoint, Dest, DestLen))
			{
				// Could not write data, bail out
				return;
			}
		}
	}

	template <typename DestBufferType>
	static bool WriteCodepointToBuffer(const uint32 Codepoint, DestBufferType& Dest, int32& DestLen)
	{
		int32 WrittenChars = Utf16FromCodepoint(Codepoint, Dest, DestLen);
		if (WrittenChars < 1)
		{
			return false;
		}

		Dest += WrittenChars;
		DestLen -= WrittenChars;
		return true;
	}
};

template<typename InFromType, typename InToType>
class TUTF16ToUTF32_Convert
{
	static_assert(sizeof(InFromType) == 2, "FromType must be 2 bytes!");
	static_assert(sizeof(InToType) == 4, "ToType must be 4 bytes!");

public:
	typedef InFromType FromType;
	typedef InToType   ToType;

	/**
	 * Converts the UTF-16 string to UTF-32.
	 *
	 * @param Dest      The destination buffer of the converted string.
	 * @param DestLen   The length of the destination buffer.
	 * @param Source    The source string to convert.
	 * @param SourceLen The length of the source string.
	 */
	template <
		typename SrcBufferType
		UE_REQUIRES(TIsCharEncodingCompatibleWith_V<SrcBufferType, FromType>)
	>
	static UE_FORCEINLINE_HINT void Convert(ToType* Dest, const int32 DestLen, const SrcBufferType* Source, const int32 SourceLen)
	{
		Convert_Impl(Dest, DestLen, (const FromType*)Source, SourceLen);
	}

	/**
	 * Determines the length of the converted string.
	 *
	 * @param Source string to read and determine amount of ToType chars to represent
	 * @param SourceLen Length of source string; we will not read past this amount, even if the characters tell us to
	 * @return The length of the string in UTF-32 characters.
	 */
	template <
		typename SrcBufferType
		UE_REQUIRES(TIsCharEncodingCompatibleWith_V<SrcBufferType, FromType>)
	>
	[[nodiscard]] static int32 ConvertedLength(const SrcBufferType* Source, const int32 SourceLen)
	{
		UE::Core::Private::FCountingOutputIterator Dest;
		Convert_Impl(Dest, MAX_int32, (const FromType*)Source, SourceLen);

		return Dest.GetCount();
	}

private:
	[[nodiscard]] static uint32 CodepointFromUtf16(const FromType*& SourceString, const uint32 SourceLengthRemaining)
	{
		checkSlow(SourceLengthRemaining > 0);

		const FromType* CodeUnitPtr = SourceString;

		uint32 Codepoint = *CodeUnitPtr;

		// Check if this character is a high-surrogate
		if (StringConv::IsHighSurrogate(Codepoint))
		{
			// Ensure our string has enough characters to read from
			if (SourceLengthRemaining < 2)
			{
				// Skip to end and write out a single char (we always have room for at least 1 char)
				SourceString += SourceLengthRemaining;
				return UNICODE_BOGUS_CHAR_CODEPOINT;
			}

			const uint16 HighSurrogate = (uint16)Codepoint;
			Codepoint = *(++CodeUnitPtr);

			// If our High Surrogate is set, check if this character is the matching low-surrogate
			if (StringConv::IsLowSurrogate(Codepoint))
			{
				const uint16 LowSurrogate = (uint16)Codepoint;

				// Combine our high and low surrogates together to a single Unicode codepoint
				Codepoint = StringConv::EncodeSurrogate(HighSurrogate, LowSurrogate);

				SourceString += 2;  // skip to next possible start of codepoint.
				return Codepoint;
			}
			else
			{
				++SourceString; // Did not find matching low-surrogate, write out a bogus character and continue
				return UNICODE_BOGUS_CHAR_CODEPOINT;
			}
		}
		else if (StringConv::IsLowSurrogate(Codepoint))
		{
			// Unpaired low surrogate
			++SourceString;
			return UNICODE_BOGUS_CHAR_CODEPOINT;
		}
		else
		{
			// Single codepoint
			++SourceString;
			return Codepoint;
		}
	}

	/**
	 * Read Source string, converting the data from UTF-16 into UTF-32, and placing these in the Destination
	 */
	template <typename DestBufferType>
	static void Convert_Impl(DestBufferType& ConvertedBuffer, int32 DestLen, const FromType* Source, const int32 SourceLen)
	{
		const FromType* SourceEnd = Source + SourceLen;
		while (Source < SourceEnd && DestLen > 0)
		{
			// Read our codepoint, advancing the source pointer
			uint32 Codepoint = CodepointFromUtf16(Source, SourceEnd - Source);

			*(ConvertedBuffer++) = Codepoint;
			--DestLen;
		}
	}
};

/**
 * Class takes one type of string and converts it to another. The class includes a
 * chunk of presized memory of the destination type. If the presized array is
 * too small, it mallocs the memory needed and frees on the class going out of
 * scope.
 */
template<typename Converter, int32 DefaultConversionSize = DEFAULT_STRING_CONVERSION_SIZE>
class TStringConversion : private Converter, private TInlineAllocator<DefaultConversionSize>::template ForElementType<typename Converter::ToType>
{
	typedef typename TInlineAllocator<DefaultConversionSize>::template ForElementType<typename Converter::ToType> AllocatorType;

	typedef typename Converter::FromType FromType;
	typedef typename Converter::ToType   ToType;

	using LegacyFromType = decltype(StringConv::GetLegacyFromType<Converter>(nullptr));

	/**
	 * Converts the data by using the Convert() method on the base class
	 */
	void Init(const FromType* Source, int32 SourceLen, UE::Core::Private::ENullTerminatedString NullTerminated)
	{
		int32 NullOffset = (NullTerminated == UE::Core::Private::ENullTerminatedString::Yes) ? 1 : 0;

		StringLength = Converter::ConvertedLength(Source, SourceLen);

		int32 BufferSize = StringLength + NullOffset;

		AllocatorType::ResizeAllocation(0, BufferSize, sizeof(ToType));

		Ptr = (ToType*)AllocatorType::GetAllocation();
		Converter::Convert(Ptr, BufferSize, Source, SourceLen + NullOffset);
	}

public:
	// Non-copyable, non-movable
	TStringConversion(TStringConversion&&) = delete;
	TStringConversion(const TStringConversion&) = delete;
	TStringConversion& operator=(TStringConversion&&) = delete;
	TStringConversion& operator=(const TStringConversion&) = delete;

	template <
		typename SrcBufferType
		UE_REQUIRES(TIsCharEncodingCompatibleWith_V<SrcBufferType, FromType>)
	>
	[[nodiscard]] explicit TStringConversion(const SrcBufferType* Source)
	{
		if (Source)
		{
			Init((const FromType*)Source, TCString<FromType>::Strlen((const FromType*)Source), UE::Core::Private::ENullTerminatedString::Yes);
		}
		else
		{
			Ptr = nullptr;
			StringLength = 0;
		}
	}
	[[nodiscard]] explicit TStringConversion(const LegacyFromType* Source)
		: TStringConversion((const FromType*)Source)
	{
	}

	template <
		typename SrcBufferType
		UE_REQUIRES(TIsCharEncodingCompatibleWith_V<SrcBufferType, FromType>)
	>
	[[nodiscard]] TStringConversion(const SrcBufferType* Source, int32 SourceLen)
	{
		if (Source)
		{
			UE::Core::Private::ENullTerminatedString NullTerminated = UE::Core::Private::ENullTerminatedString::No;
			if (SourceLen > 0 && ((const FromType*)Source)[SourceLen-1] == 0)
			{
				// Given buffer is null-terminated
				NullTerminated = UE::Core::Private::ENullTerminatedString::Yes;
				SourceLen -= 1;
			}

			Init((const FromType*)Source, SourceLen, NullTerminated);
		}
		else
		{
			Ptr = nullptr;
			StringLength = 0;
		}
	}
	[[nodiscard]] TStringConversion(const LegacyFromType* Source, int32 SourceLen)
		: TStringConversion((const FromType*)Source, SourceLen)
	{
	}

	[[nodiscard]] explicit TStringConversion(UE::Core::Private::FFromStrCast, const FromType* Source, int32 SourceLen)
	{
		Init(Source, SourceLen, UE::Core::Private::ENullTerminatedString::No);
	}

	/**
	 * Construct from a compatible character range such as TStringView or TStringBuilder.
	 */
	template <
		typename FromRangeType,
		typename FromRangeCharType = std::remove_cv_t<std::remove_pointer_t<decltype(GetData(std::declval<FromRangeType>()))>>
		UE_REQUIRES(
			TIsContiguousContainer<FromRangeType>::Value &&
			!TIsArray<std::remove_reference_t<FromRangeType>>::Value &&
			TIsCharEncodingCompatibleWith<FromRangeCharType, FromType>::Value
		)
	>
	[[nodiscard]] TStringConversion(FromRangeType&& Source)
		: TStringConversion((const FromType*)GetData(Source), GetNum(Source))
	{
	}

	/**
	 * Accessor for the converted string.
	 * @note The string may not be null-terminated if constructed from an explicitly sized buffer that didn't include the null-terminator.
	 *
	 * @return A const pointer to the converted string.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const ToType* Get() const UE_LIFETIMEBOUND
	{
		return Ptr;
	}

	/**
	 * Length of the converted string.
	 *
	 * @return The number of characters in the converted string, excluding any null terminator.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Length() const
	{
		return StringLength;
	}

private:
	ToType* Ptr;
	int32   StringLength;
};

template <typename Converter, int32 DefaultConversionSize>
[[nodiscard]] inline auto GetData(const TStringConversion<Converter, DefaultConversionSize>& Conversion) -> decltype(Conversion.Get())
{
	return Conversion.Get();
}

template <typename Converter, int32 DefaultConversionSize>
[[nodiscard]] inline auto GetNum(const TStringConversion<Converter, DefaultConversionSize>& Conversion) -> decltype(Conversion.Length())
{
	return Conversion.Length();
}

template <typename Converter, int32 DefaultConversionSize>
struct TIsContiguousContainer<TStringConversion<Converter, DefaultConversionSize>>
{
	static constexpr bool Value = true;
};

template <typename Converter, int32 DefaultConversionSize>
struct TElementType<TStringConversion<Converter, DefaultConversionSize>>
{
	using Type = typename Converter::ToType;
};


/**
 * Class takes one type of string and and stores it as-is.
 * For API compatibility with TStringConversion when the To and From types are already in the same format.
 */
template<typename InFromType, typename InToType = InFromType>
class TStringPointer
{
	static_assert(sizeof(InFromType) == sizeof(InToType), "FromType must be the same size as ToType!");

public:
	typedef InFromType FromType;
	typedef InToType   ToType;

public:
	template <
		typename SrcBufferType
		UE_REQUIRES(TIsCharEncodingCompatibleWith_V<SrcBufferType, FromType>)
	>
	[[nodiscard]] explicit TStringPointer(const SrcBufferType* Source)
	{
		if (Source)
		{
			Ptr = (const ToType*)Source;
			StringLength = -1; // Length calculated on-demand
		}
		else
		{
			Ptr = nullptr;
			StringLength = 0;
		}
	}

	template <
		typename SrcBufferType
		UE_REQUIRES(TIsCharEncodingCompatibleWith_V<SrcBufferType, FromType>)
	>
	[[nodiscard]] explicit TStringPointer(const SrcBufferType* Source, int32 SourceLen)
	{
		if (Source)
		{
			if (SourceLen > 0 && ((const FromType*)Source)[SourceLen-1] == 0)
			{
				// Given buffer is null-terminated
				SourceLen -= 1;
			}

			Ptr = (const ToType*)Source;
			StringLength = SourceLen;
		}
		else
		{
			Ptr = nullptr;
			StringLength = 0;
		}
	}

	// This constructor is only intended to be used by StrCast, not end users
	[[nodiscard]] explicit TStringPointer(UE::Core::Private::FFromStrCast, const FromType* Source, int32 SourceLen)
		: Ptr((const ToType*)Source)
		, StringLength(SourceLen)
	{
	}

	/**
	 * Construct from a compatible character range such as TStringView or TStringBuilder.
	 */
	template <
		typename FromRangeType,
		typename FromRangeCharType = std::remove_cv_t<std::remove_pointer_t<decltype(GetData(std::declval<FromRangeType>()))>>
		UE_REQUIRES(
			TIsContiguousContainer<FromRangeType>::Value &&
			!TIsArray<std::remove_reference_t<FromRangeType>>::Value &&
			TIsCharEncodingCompatibleWith<FromRangeCharType, FromType>::Value
		)
	>
	[[nodiscard]] TStringPointer(FromRangeType&& Source)
		: TStringPointer((const FromType*)GetData(Source), GetNum(Source))
	{
	}

	/**
	 * Move constructor
	 */
	[[nodiscard]] TStringPointer(TStringPointer&& Other) = default;

	/**
	 * Accessor for the string.
	 * @note The string may not be null-terminated if constructed from an explicitly sized buffer that didn't include the null-terminator.
	 * @note Setting to UE_LIFETIMEBOUND even though TStringPointer doesn't own the pointer, because we want StringCast to give consistent warnings no matter what implementation choice it makes
	 *
	 * @return A const pointer to the string.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const ToType* Get() const UE_LIFETIMEBOUND
	{
		return Ptr;
	}

	/**
	 * Length of the string.
	 *
	 * @return The number of characters in the string, excluding any null terminator.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Length() const
	{
		return StringLength == -1 ? TCString<ToType>::Strlen(Ptr) : StringLength;
	}

private:
	// Non-copyable
	TStringPointer(const TStringPointer&) = delete;
	TStringPointer& operator=(const TStringPointer&) = delete;

	const ToType* Ptr;
	int32 StringLength;
};

template <typename FromType, typename ToType>
[[nodiscard]] inline auto GetData(const TStringPointer<FromType, ToType>& Pointer) -> decltype(Pointer.Get())
{
	return Pointer.Get();
}

template <typename FromType, typename ToType>
[[nodiscard]] inline auto GetNum(const TStringPointer<FromType, ToType>& Pointer) -> decltype(Pointer.Length())
{
	return Pointer.Length();
}

template <typename FromType, typename ToType>
struct TIsContiguousContainer<TStringPointer<FromType, ToType>>
{
	static constexpr bool Value = true;
};

template <typename FromType, typename ToType>
struct TElementType<TStringPointer<FromType, ToType>>
{
	using Type = ToType;
};


/**
 * NOTE: The objects these macros declare have very short lifetimes. They are
 * meant to be used as parameters to functions. You cannot assign a variable
 * to the contents of the converted string as the object will go out of
 * scope and the string released.
 *
 * NOTE: The parameter you pass in MUST be a proper string, as the parameter
 * is typecast to a pointer. If you pass in a char, not char* it will compile
 * and then crash at runtime.
 *
 * Usage:
 *
 *		SomeApi(TCHAR_TO_ANSI(SomeUnicodeString));
 *
 *		const char* SomePointer = TCHAR_TO_ANSI(SomeUnicodeString); <--- Bad!!!
 */

/*********************************************************************
 * NOTE:
 * The following converting types and macros should be considered
 * deprecated and StringCast should be used instead.
 *
 * The reasons for deprecation include:
 *
 * - char is the wrong type for representing a UTF-8 encoding.  UTF8CHAR should be used now, which will become char8_t from C++20 onwards.
 * - The macros hide a reinterpret_cast of the argument, which is both misleading and dangerous.
 * - The const return value is silently cast away, which is also misleading and dangerous.
 * - The macros are frequently used to initialize a local variable, which ends up pointing at deleted memory.
 * - The names are hardcoded with the types they cast from and to, making them unusable in generic contexts.
 *
 * Examples of migration:
 *   ANSICHAR* AnsiCharPtr = ...;
 *   UTF8CHAR* Utf8CharPtr = ...;
 *   TCHAR*    TCharPtr    = ...;
 *
 * Before:
 *   FUTF8ToTCHAR Conv1(AnsiCharPtr); // Use Conv1.Get() and Conv1.Length()
 *   FUTF8ToTCHAR Conv2(Utf8CharPtr); // Use Conv2.Get() and Conv2.Length()
 *   FunctionTakingTCHARPtr(ANSI_TO_TCHAR(AnsiCharPtr));
 *   Printf(TEXT("Narrow string: %s"), ANSI_TO_TCHAR(AnsiCharPtr));
 *   Printf(TEXT("UTF-8 string: %s"), UTF8_TO_TCHAR(Utf8CharPtr));
 *
 * After:
 *   auto Conv1 = StringCast<TCHAR>((const UTF8CHAR*)AnsiCharPtr); // Use Conv1.Get() and Conv1.Length()
 *   auto Conv2 = StringCast<TCHAR>(UTF8CharPtr);                  // Use Conv2.Get() and Conv2.Length()
 *   FunctionTakingTCHARPtr(StringCast<TCHAR>(AnsiCharPtr).Get());
 *   Printf(TEXT("Narrow string: %hs"), AnsiCharPtr);
 *   Printf(TEXT("Narrow string: %hs"), Utf8CharPtr);
 *********************************************************************/
using FTCHARToUTF8 /*UE_DEPRECATED(5.xx, "FTCHARToUTF8(PtrToTChar) is deprecated, please use StringCast<UTF8CHAR>(PtrToTChar) instead.")*/ = TStringConversion<UE::Core::Private::FTCHARToUTF8_Convert>;
using FUTF8ToTCHAR /*UE_DEPRECATED(5.xx, "FUTF8ToTCHAR(PtrToUTF8) is deprecated, please use StringCast<TCHAR>(PtrToUTF8Char) instead.")*/ = TStringConversion<FUTF8ToTCHAR_Convert>;

#define TCHAR_TO_ANSI(str) /*UE_DEPRECATED_MACRO(5.xx, "TCHAR_TO_ANSI(Ptr) is deprecated, please use StringCast<ANSICHAR>(PtrToTChar) instead.")*/ (ANSICHAR*)StringCast<ANSICHAR>(static_cast<const TCHAR*>(str)).Get()
#define ANSI_TO_TCHAR(str) /*UE_DEPRECATED_MACRO(5.xx, "ANSI_TO_TCHAR(Ptr) is deprecated, please use StringCast<TCHAR>(PtrToAnsiChar) instead.")*/ (TCHAR*)StringCast<TCHAR>(static_cast<const ANSICHAR*>(str)).Get()
#define TCHAR_TO_UTF8(str) /*UE_DEPRECATED_MACRO(5.xx, "TCHAR_TO_UTF8(Ptr) is deprecated, please use StringCast<UTF8CHAR>(PtrToTChar) instead.")*/ (ANSICHAR*)FTCHARToUTF8((const TCHAR*)str).Get()
#define UTF8_TO_TCHAR(str) /*UE_DEPRECATED_MACRO(5.xx, "UTF8_TO_TCHAR(Ptr) is deprecated, please use StringCast<TCHAR>(PtrToUTF8Char) instead.")*/ (TCHAR*)FUTF8ToTCHAR((const ANSICHAR*)str).Get()


////////////////////////////////////////////////////////////////////
// NOTE:                                                          //
// The following converting types and macros cannot be deprecated //
// until FPlatformString has proper handling of UTF16CHAR and     //
// UTF32CHAR.                                                     //
////////////////////////////////////////////////////////////////////

#if PLATFORM_TCHAR_IS_UTF8CHAR

typedef TStringConversion<TStringConvert<TCHAR, UTF16CHAR>> FTCHARToUTF16;
typedef TStringConversion<TStringConvert<UTF16CHAR, TCHAR>> FUTF16ToTCHAR;
#define TCHAR_TO_UTF16(str) (UTF16CHAR*)FTCHARToUTF16((const TCHAR*)str).Get()
#define UTF16_TO_TCHAR(str) (TCHAR*)FUTF16ToTCHAR((const UTF16CHAR*)str).Get()

typedef TStringConversion<TStringConvert<TCHAR, UTF32CHAR>> FTCHARToUTF32;
typedef TStringConversion<TStringConvert<UTF32CHAR, TCHAR>> FUTF32ToTCHAR;
#define TCHAR_TO_UTF32(str) (UTF32CHAR*)FTCHARToUTF32((const TCHAR*)str).Get()
#define UTF32_TO_TCHAR(str) (TCHAR*)FUTF32ToTCHAR((const UTF32CHAR*)str).Get()

#else

static_assert(sizeof(TCHAR) == sizeof(UTF16CHAR), "TCHAR and UTF16CHAR are expected to be the same size for inline conversion!");
typedef TStringPointer<TCHAR, UTF16CHAR> FTCHARToUTF16;
typedef TStringPointer<UTF16CHAR, TCHAR> FUTF16ToTCHAR;
#define TCHAR_TO_UTF16(str) (UTF16CHAR*)(str)
#define UTF16_TO_TCHAR(str) (TCHAR*)(str)

typedef TStringConversion<TUTF16ToUTF32_Convert<TCHAR, UTF32CHAR>> FTCHARToUTF32;
typedef TStringConversion<TUTF32ToUTF16_Convert<UTF32CHAR, TCHAR>> FUTF32ToTCHAR;
#define TCHAR_TO_UTF32(str) (UTF32CHAR*)FTCHARToUTF32((const TCHAR*)str).Get()
#define UTF32_TO_TCHAR(str) (TCHAR*)FUTF32ToTCHAR((const UTF32CHAR*)str).Get()

#endif

// special handling for going from char16_t to wchar_t for third party libraries that need wchar_t
#if PLATFORM_TCHAR_IS_CHAR16 && PLATFORM_WCHAR_IS_4_BYTES
typedef TStringConversion<TUTF16ToUTF32_Convert<TCHAR, wchar_t>> FTCHARToWChar;
typedef TStringConversion<TUTF32ToUTF16_Convert<wchar_t, TCHAR>> FWCharToTCHAR;
#define TCHAR_TO_WCHAR(str) (wchar_t*)FTCHARToWChar((const TCHAR*)str).Get()
#define WCHAR_TO_TCHAR(str) (TCHAR*)FWCharToTCHAR((const wchar_t*)str).Get()

#elif PLATFORM_TCHAR_IS_UTF8CHAR

typedef TStringConversion<TStringConvert<TCHAR, wchar_t>> FTCHARToWChar;
typedef TStringConversion<TStringConvert<wchar_t, TCHAR>> FWCharToTCHAR;
#define TCHAR_TO_WCHAR(str) (wchar_t*)FTCHARToUTF32((const TCHAR*)str).Get()
#define WCHAR_TO_TCHAR(str) (TCHAR*)FWCharToTCHAR((const wchar_t*)str).Get()

#else

static_assert(sizeof(TCHAR) == sizeof(wchar_t), "TCHAR and wchar_t are expected to be the same size for inline conversion! PLATFORM_WCHAR_IS_4_BYTES is not configured correctly for this platform.");
typedef TStringPointer<TCHAR, wchar_t> FTCHARToWChar;
typedef TStringPointer<wchar_t, TCHAR> FWCharToTCHAR;
#define TCHAR_TO_WCHAR(str) (wchar_t*)(str)
#define WCHAR_TO_TCHAR(str) (TCHAR*)(str)

#endif

/**
 * StringCast example usage:
 *
 * void Func(const FString& Str, const TCHAR* MediumLong, FStringView MediumLongSV)
 * {
 *     // Basic version
 *     auto Src = StringCast<ANSICHAR>(*Str);
 *     const ANSICHAR* Ptr = Src.Get(); // Ptr is a pointer to an ANSICHAR representing the potentially-converted string data.
 *     // Avoid calling strlen
 *     auto NoStrlenString = StringCast<ANSICHAR>(MediumLongSV.GetData(), MediumLongSV.Len());
 *     // Specify the static allocation size when maximum length is known
 *     auto NoAllocSrc = StringCast<ANSICHAR, 1024>(MediumLong);
 *     // Avoid calling strlen and also specify the allocation size
 *     auto NoAllocNoStrlenSrc = StringCast<ANSICHAR, 1024>(MediumLongSV.GetData(), MediumLongSV.Len());
 * }
 *
 */

/**
 * Creates an object which acts as a source of a given string type.  See example above.
 *
 * StringCast expects correctly-typed strings.  If a cast is attempted with a char* or ANSICHAR*,
 * and the string contains characters that are non-ASCII, including UTF-8 code units outside of
 * the 7-bit ASCII range, then those values will fail to convert and a bogus char will be written
 * in their place.
 *
 * If a conversion from UTF-8 is desired, the pointer should be cast to UTF8CHAR* before being
 * passed to StringCast.
 *
 * Similarly, doing a StringCast<char> or StringCast<ANSICHAR> on a Unicode string will only
 * successfully convert Unicode characters which already lie in the ASCII range.  For converting
 * to UTF-8, StringCast<UTF8CHAR> should be used.
 *
 * The source string must not be modified or destroyed until after the result of this function
 * has been destroyed.
 *
 * @param Str The null-terminated source string to convert.
 */
template <typename To, int32 DefaultConversionSize = DEFAULT_STRING_CONVERSION_SIZE, typename From>
[[nodiscard]] inline auto StringCast(const From* Str)
{
	if constexpr (TIsCharEncodingCompatibleWith_V<From, To>)
	{
		return TStringPointer<To>((const To*)Str);
	}
	else
	{
		return TStringConversion<TStringConvert<From, To>, DefaultConversionSize>(Str);
	}
}

/**
 * Creates an object which acts as a source of a given string type.  See example above.
 *
 * StringCast expects correctly-typed strings.  If a cast is attempted with a char* or ANSICHAR*,
 * and the string contains characters that are non-ASCII, including UTF-8 code units outside of
 * the 7-bit ASCII range, then those values will fail to convert and a bogus char will be written
 * in their place.
 *
 * If a conversion from UTF-8 is desired, the pointer should be cast to UTF8CHAR* before being
 * passed to StringCast.
 *
 * Similarly, doing a StringCast<char> or StringCast<ANSICHAR> on a Unicode string will only
 * successfully convert Unicode characters which already lie in the ASCII range.  For converting
 * to UTF-8, StringCast<UTF8CHAR> should be used.
 *
 * The source string must not be modified or destroyed until after the result of this function
 * has been destroyed.
 *
 * @param Str The source string to convert, not necessarily null-terminated.
 * @param Len The number of From elements in Str.
 */
template <typename To, int32 DefaultConversionSize = DEFAULT_STRING_CONVERSION_SIZE, typename From>
[[nodiscard]] inline auto StringCast(const From* Str, int32 Len)
{
	if constexpr (TIsCharEncodingCompatibleWith_V<From, To>)
	{
		return TStringPointer<To>((const To*)Str, Len);
	}
	else
	{
		return TStringConversion<TStringConvert<From, To>, DefaultConversionSize>(Str, Len);
	}
}

/**
 * Creates an object which acts as a source of a given string type.  See example above.
 *
 * This is intended as the long-term replacement for StringCast, which doesn't cope well
 * with zeros mid-string, which it can interpret as a null-terminators and give surprising behavior.
 *
 * StrCast expects correctly-typed strings.  If a cast is attempted with a char* or ANSICHAR*,
 * and the string contains characters that are non-ASCII, including UTF-8 code units outside of
 * the 7-bit ASCII range, then those values will fail to convert and a bogus char will be written
 * in their place.
 *
 * If a conversion from UTF-8 is desired, the pointer should be cast to UTF8CHAR* before being
 * passed to StrCast.
 *
 * Similarly, doing a StrCast<char> or StrCast<ANSICHAR> on a Unicode string will only
 * successfully convert Unicode characters which already lie in the ASCII range.  For converting
 * to UTF-8, StrCast<UTF8CHAR> should be used.
 *
 * The source string must not be modified or destroyed until after the result of this function
 * has been destroyed.
 *
 * @param Str A pointer to the start of the string to convert.  Must be non-null if Len is non-zero.
 * @param Len The number of From elements in Str.  Must be non-negative.
 */
template <typename To, int32 DefaultConversionSize = DEFAULT_STRING_CONVERSION_SIZE, typename From>
[[nodiscard]] inline auto StrCast(const From* Str, int32 Len)
{
	if constexpr (TIsCharEncodingCompatibleWith_V<From, To>)
	{
		return TStringPointer<To>(UE::Core::Private::FFromStrCast{}, (const To*)Str, Len);
	}
	else
	{
		return TStringConversion<TStringConvert<From, To>, DefaultConversionSize>(UE::Core::Private::FFromStrCast{}, Str, Len);
	}
}


/**
 * Casts one fixed-width char type into another.
 *
 * @param Ch The character to convert.
 * @return The converted character.
 */
template <typename To, typename From>
[[nodiscard]] inline To CharCast(From Ch)
{
	To Result;
	FPlatformString::Convert(&Result, 1, &Ch, 1);
	return Result;
}

/**
 * This class is returned by StringPassthru and is not intended to be used directly.
 */
template <typename ToType, typename FromType, int32 DefaultConversionSize = DEFAULT_STRING_CONVERSION_SIZE>
class TStringPassthru : private TInlineAllocator<DefaultConversionSize>::template ForElementType<FromType>
{
	typedef typename TInlineAllocator<DefaultConversionSize>::template ForElementType<FromType> AllocatorType;

public:
	[[nodiscard]] inline TStringPassthru(ToType* InDest, int32 InDestLen, int32 InSrcLen)
		: Dest   (InDest)
		, DestLen(InDestLen)
		, SrcLen (InSrcLen)
	{
		AllocatorType::ResizeAllocation(0, SrcLen, sizeof(FromType));
	}

	[[nodiscard]] UE_FORCEINLINE_HINT TStringPassthru(TStringPassthru&& Other)
	{
		AllocatorType::MoveToEmpty(Other);
	}

	inline void Apply() const
	{
		const FromType* Source = (const FromType*)AllocatorType::GetAllocation();
		check(FPlatformString::ConvertedLength<ToType>(Source, SrcLen) <= DestLen);
		FPlatformString::Convert(Dest, DestLen, Source, SrcLen);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT FromType* Get() UE_LIFETIMEBOUND
	{
		return (FromType*)AllocatorType::GetAllocation();
	}

private:
	// Non-copyable
	TStringPassthru(const TStringPassthru&);
	TStringPassthru& operator=(const TStringPassthru&);

	ToType* Dest;
	int32   DestLen;
	int32   SrcLen;
};

// This seemingly-pointless class is intended to be API-compatible with TStringPassthru
// and is returned by StringPassthru when no string conversion is necessary.
template <typename T>
class TPassthruPointer
{
public:
	[[nodiscard]] UE_FORCEINLINE_HINT explicit TPassthruPointer(T* InPtr)
		: Ptr(InPtr)
	{
	}

	// @note Setting to UE_LIFETIMEBOUND even though TPassthruPointer doesn't own the pointer, because we want StringMemoryPassthru to give consistent warnings no matter what implementation choice it makes
	[[nodiscard]] UE_FORCEINLINE_HINT T* Get() const UE_LIFETIMEBOUND
	{
		return Ptr;
	}

	UE_FORCEINLINE_HINT void Apply() const
	{
	}

private:
	T* Ptr;
};

/**
 * Allows the efficient conversion of strings by means of a temporary memory buffer only when necessary.  Intended to be used
 * when you have an API which populates a buffer with some string representation which is ultimately going to be stored in another
 * representation, but where you don't want to do a conversion or create a temporary buffer for that string if it's not necessary.
 *
 * Intended use:
 *
 * // Populates the buffer Str with StrLen characters.
 * void SomeAPI(APICharType* Str, int32 StrLen);
 *
 * void Func(DestChar* Buffer, int32 BufferSize)
 * {
 *     // Create a passthru.  This takes the buffer (and its size) which will ultimately hold the string, as well as the length of the
 *     // string that's being converted, which must be known in advance.
 *     // An explicit template argument is also passed to indicate the character type of the source string.
 *     // Buffer must be correctly typed for the destination string type.
 *     auto Passthru = StringMemoryPassthru<APICharType>(Buffer, BufferSize, SourceLength);
 *
 *     // Passthru.Get() returns an APICharType buffer pointer which is guaranteed to be SourceLength characters in size.
 *     // It's possible, and in fact intended, for Get() to return the same pointer as Buffer if DestChar and APICharType are
 *     // compatible string types.  If this is the case, SomeAPI will write directly into Buffer.  If the string types are not
 *     // compatible, Get() will return a pointer to some temporary memory which allocated by and owned by the passthru.
 *     SomeAPI(Passthru.Get(), SourceLength);
 *
 *     // If the string types were not compatible, then the passthru used temporary storage, and we need to write that back to Buffer.
 *     // We do that with the Apply call.  If the string types were compatible, then the data was already written to Buffer directly
 *     // and so Apply is a no-op.
 *     Passthru.Apply();
 *
 *     // Now Buffer holds the data output by SomeAPI, already converted if necessary.
 * }
 */
template <typename From, typename To, int32 DefaultConversionSize = DEFAULT_STRING_CONVERSION_SIZE>
[[nodiscard]] inline auto StringMemoryPassthru(To* Buffer, int32 BufferSize, int32 SourceLength)
{
	if constexpr (TIsCharEncodingCompatibleWith_V<From, To>)
	{
		check(SourceLength <= BufferSize);
		return TPassthruPointer<From>((From*)Buffer);
	}
	else
	{
		return TStringPassthru<To, From, DefaultConversionSize>(Buffer, BufferSize, SourceLength);
	}
}

template <typename ToType, typename FromType>
[[nodiscard]] inline TArray<ToType> StringToArray(const FromType* Src, int32 SrcLen)
{
	int32 DestLen = FPlatformString::ConvertedLength<ToType>(Src, SrcLen);

	TArray<ToType> Result;
	Result.AddUninitialized(DestLen);
	FPlatformString::Convert(Result.GetData(), DestLen, Src, SrcLen);

	return Result;
}

template <typename ToType, typename FromType>
[[nodiscard]] UE_FORCEINLINE_HINT TArray<ToType> StringToArray(const FromType* Str)
{
	return StringToArray<ToType>(Str, TCString<FromType>::Strlen(Str) + 1);
}
