// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_exports

#include "CoreTypes.h"
#include "Concepts/CharType.h"
#include "Traits/ElementType.h"
#include "Traits/IsContiguousContainer.h"

// String Builder

template <typename CharType> class TStringBuilderBase;
template <typename CharType, int32 BufferSize> class TStringBuilderWithBuffer;
template <typename CharType> class TExternalStringBuilder;

template <typename CharType> struct TIsContiguousContainer<TStringBuilderBase<CharType>> { static constexpr bool Value = true; };
template <typename CharType, int32 BufferSize> struct TIsContiguousContainer<TStringBuilderWithBuffer<CharType, BufferSize>> { static constexpr bool Value = true; };
template <typename CharType> struct TIsContiguousContainer<TExternalStringBuilder<CharType>> { static constexpr bool Value = true; };

/** The base string builder type for TCHAR. */
using FStringBuilderBase = TStringBuilderBase<TCHAR>;
/** The base string builder type for ANSICHAR. */
using FAnsiStringBuilderBase = TStringBuilderBase<ANSICHAR>;
/** The base string builder type for WIDECHAR. */
using FWideStringBuilderBase = TStringBuilderBase<WIDECHAR>;
/** The base string builder type for UTF8CHAR. */
using FUtf8StringBuilderBase = TStringBuilderBase<UTF8CHAR>;

/** An extendable string builder for TCHAR. */
template <int32 BufferSize> using TStringBuilder = TStringBuilderWithBuffer<TCHAR, BufferSize>;
/** An extendable string builder for ANSICHAR. */
template <int32 BufferSize> using TAnsiStringBuilder = TStringBuilderWithBuffer<ANSICHAR, BufferSize>;
/** An extendable string builder for WIDECHAR. */
template <int32 BufferSize> using TWideStringBuilder = TStringBuilderWithBuffer<WIDECHAR, BufferSize>;
/** An extendable string builder for UTF8CHAR. */
template <int32 BufferSize> using TUtf8StringBuilder = TStringBuilderWithBuffer<UTF8CHAR, BufferSize>;

// String View

template <UE::CCharType T>
class TStringView;

template <UE::CCharType CharType> struct TIsContiguousContainer<TStringView<CharType>> { static constexpr bool Value = true; };

using FStringView = TStringView<TCHAR>;
using FAnsiStringView = TStringView<ANSICHAR>;
using FWideStringView = TStringView<WIDECHAR>;
using FUtf8StringView = TStringView<UTF8CHAR>;

// Shared String

namespace UE
{

template <typename CharType> class TSharedString;

using FSharedString = TSharedString<TCHAR>;
using FAnsiSharedString = TSharedString<ANSICHAR>;
using FWideSharedString = TSharedString<WIDECHAR>;
using FUtf8SharedString = TSharedString<UTF8CHAR>;

} // UE

template <typename CharType>
struct TIsContiguousContainer<UE::TSharedString<CharType>>
{
	static constexpr bool Value = true;
};

// IWYU pragma: end_exports
