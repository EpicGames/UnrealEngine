// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Requires.h"
#include "Traits/ElementType.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "Traits/IsContiguousContainer.h"
#include "Traits/IsImplicitlyConstructible.h"
#include "Traits/IsTString.h"
#include <type_traits>

// A type which wraps a TString parameter but fails (SFINAE) construction if passed a string argument of an incompatible character type.
// This will allow FString (TString<WIDECHAR>) and FUtf8String (TString<UTF8CHAR>) overloads to sit next to each other and direct
// pointer, array or other TString arguments to the right overload.
//
// This is important because of the legacy of UE's string constructors, which takes anything that looks like a string, of any character
// width, so overloading - say - FString and FUtf8String will cause overload resolution problems.
//
// The argument should be copied/moved into the real TString instance in order to use it, as this type has no string manipulation
// ability.
//
// This argument is equivalent to a pass-by-value toboth the lvalue/rvalue overload case.  Once , and can be converted back to regular
// const TString&/TString&& overloads once overloading is no longer required.
//
// Example:
//   void Func(TStringOverload<FWideString>); // called by Func(WideString), Func(MoveTemp(WideString)) or Func(TEXT("String"))
//   void Func(TStringOverload<FUtf8String>); // called by Func(Utf8String), Func(MoveTemp(Utf8String)), Func(UTF8TEXT("String")) or Func("String") or Func({})
template <typename StringType>
struct TStringOverload
{
private:
	static_assert(std::is_same_v<StringType, std::remove_cv_t<StringType>> && TIsTString_V<StringType>, "TStringOverload expects an unqualified string object type");

	using CharType = typename StringType::ElementType;

	template <typename ArgType>
	static constexpr bool IsValidArgType()
	{
		using DecayedArgType = std::decay_t<ArgType>;

		if constexpr (!TIsImplicitlyConstructible_V<StringType, DecayedArgType>)
		{
			return false;
		}
		else if constexpr (TIsContiguousContainer<DecayedArgType>::Value)
		{
			return TIsCharEncodingCompatibleWith_V<TElementType_T<DecayedArgType>, CharType>;
		}
		else if constexpr (std::is_pointer_v<DecayedArgType>)
		{
			return TIsCharEncodingCompatibleWith_V<std::remove_const_t<std::remove_pointer_t<DecayedArgType>>, CharType>;
		}
		else
		{
			static_assert(sizeof(ArgType) == 0, "Unsupported argument type passed to TStringOverload");
			return false;
		}
	}

public:
	// This Construct allows us to default construct a TStringOverload using {}.
	// It only exists for a narrow string, because {} needs to resolve one way or another, and we want it
	// to be FUtf8String by default.
	template <
		typename T = CharType
		UE_REQUIRES(sizeof(T) == 1)
	>
	TStringOverload()
	{
	}

	template <
		typename ArgType
		UE_REQUIRES(IsValidArgType<ArgType>())
	>
	TStringOverload(ArgType&& Arg)
		: String((ArgType&&)Arg)
	{
	}

	StringType&& MoveTemp()
	{
		// Shouldn't need a const overload, because these types are only intended as by-non-const-value function parameters, and the only
		// thing you can do with it is move it into a real string object.
		return (StringType&&)String;
	}

	StringType String;
};

namespace UE::Core::Private
{
	template <typename CharType, typename ArgType>
	inline ArgType&& CheckCharType(ArgType&& Arg)
	{
		using DecayedArgType = std::decay_t<ArgType>;

		if constexpr (TIsContiguousContainer<DecayedArgType>::Value)
		{
			static_assert(std::is_same_v<const TElementType_T<DecayedArgType>, const CharType>, "Incorrect string type passed to UE_PRIVATE_TO_STRING macro");
		}
		else if constexpr (std::is_pointer_v<DecayedArgType>)
		{
			static_assert(std::is_same_v<const std::remove_pointer_t<DecayedArgType>, const CharType>, "Incorrect string type passed to UE_PRIVATE_TO_STRING macro");
		}
		else
		{
			static_assert(sizeof(ArgType) == 0, "Unsupported argument type passed to TStringOverload");
		}
		return (ArgType&&)Arg;
	}
}

// NOT FOR GENERAL USE!
//
// These macros exist to make it easier to denote where wide<->narrow conversions are needed due to legacy APIs,
// so they can be located and fixed later.
#define UE_PRIVATE_TO_UTF8_STRING(Str) FUtf8String(UE::Core::Private::CheckCharType<WIDECHAR>(Str))
#define UE_PRIVATE_TO_WIDE_STRING(Str) FWideString(UE::Core::Private::CheckCharType<UTF8CHAR>(Str))
