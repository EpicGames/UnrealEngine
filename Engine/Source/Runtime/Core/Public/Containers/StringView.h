// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Concepts/CharType.h"
#include "Concepts/CompatibleCharType.h"
#include "Concepts/CompatibleStringViewable.h"
#include "Concepts/ConvertibleTo.h"
#include "Concepts/NotCVRefTo.h"
#include "Concepts/StringViewable.h"
#include "Containers/StringFwd.h" // IWYU pragma: export
#include "HAL/UnrealMemory.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/Crc.h"
#include "Misc/CString.h"
#include "Misc/ReverseIterate.h"
#include "Misc/UEOps.h"
#include "String/Find.h"
#include "Templates/UnrealTemplate.h"
#include "Traits/ElementType.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "Traits/IsCharType.h"
#include "Traits/IsContiguousContainer.h"
#include <type_traits>

namespace UE::Core::Private
{
	template <UE::CCharType CharType>
	constexpr int32 StringLength(const CharType* InString)
	{
		int32 Count = 0;
		if (InString)
		{
			UE_IF_CONSTEVAL
			{
				while (*InString++)
				{
					++Count;
				}
			}
			else
			{
				Count = TCString<CharType>::Strlen(InString);
			}
		}
		return Count;
	}
} // UE::Core::Private

template <typename T> inline constexpr bool TIsTStringView_V                 = false;
template <typename T> inline constexpr bool TIsTStringView_V<TStringView<T>> = true;

template <typename T> inline constexpr bool TIsTStringView_V<const          T> = TIsTStringView_V<T>;
template <typename T> inline constexpr bool TIsTStringView_V<      volatile T> = TIsTStringView_V<T>;
template <typename T> inline constexpr bool TIsTStringView_V<const volatile T> = TIsTStringView_V<T>;

template <typename T>
struct TIsTStringView
{
	static constexpr bool Value = TIsTStringView_V<T>;
	static constexpr bool value = Value;
};


/**
 * A string view is a non-owning view of a range of characters.
 *
 * Ensure that the underlying string is valid for the lifetime of the string view.
 *
 * Be careful when constructing a string view from a temporary. Make a local copy if necessary.
 *
 * FStringView View = Object->GetPathName(); // Invalid
 *
 * FString PathName = Object->GetPathName(); // Valid
 * FStringView View = PathName;
 *
 * void ProcessPath(FStringView Path);       // Valid
 * ProcessPath(Object->GetPathName());
 *
 * A string view is implicitly constructible from null-terminated strings, from contiguous ranges
 * of characters such as FString and TStringBuilder, and from literals such as TEXTVIEW("...").
 *
 * A string view is cheap to copy and is meant to be passed by value. Avoid passing by reference.
 *
 * A string view is not guaranteed to represent a null-terminated string.
 *
 * Log or format a string view using UE_LOG(TEXT("%.*s"), View.Len(), View.GetData());
 *
 * A string view is a good fit for function parameters where the function has no requirements for
 * how the string is stored. A caller may use FString, FStringView, TStringBuilder, a char array,
 * a null-terminated string, or any other type which can convert to a string view.
 *
 * The UE::String namespace contains many functions that can operate on string views.
 * Most of these functions can be found in String/___.h in Core.
 *
 * @code
 *	void UseString(FStringView InString);
 *
 *	FString String("ABC");
 *	const TCHAR* CString = *String;
 *	TStringBuilder<16> StringBuilder;
 *	StringBuilder.Append(TEXT("ABC"));
 *
 *	UseString(String);
 *	UseString(CString);
 *	UseString(StringBuilder);
 *	UseString(TEXT("ABC"));
 *	UseString(TEXTVIEW("ABC"));
 * @endcode
 */
template <UE::CCharType CharType>
class TStringView
{
public:
	using ElementType = CharType;
	using ViewType = TStringView<CharType>;

public:
	/** Construct an empty view. */
	[[nodiscard]] constexpr TStringView() = default;

	/** Construct a view of the null-terminated string pointed to by InData. */
	[[nodiscard]] constexpr inline TStringView(const CharType* InData UE_LIFETIMEBOUND)
		: DataPtr(InData)
		, Size(UE::Core::Private::StringLength(InData))
	{
	}

	/** Construct a view of InSize characters beginning at InData. */
	[[nodiscard]] constexpr inline TStringView(const CharType* InData UE_LIFETIMEBOUND, int32 InSize)
		: DataPtr(InData)
		, Size(InSize)
	{
	}

	/** Construct a view of the null-terminated string pointed to by InData. */
	template <UE::CCompatibleCharType<CharType> OtherCharType>
	[[nodiscard]] inline TStringView(const OtherCharType* InData UE_LIFETIMEBOUND)
		: DataPtr((const CharType*)InData)
		, Size(UE::Core::Private::StringLength((const CharType*)InData))
	{
	}

	/** Construct a view of InSize characters beginning at InData. */
	template <UE::CCompatibleCharType<CharType> OtherCharType>
	[[nodiscard]] constexpr inline TStringView(const OtherCharType* InData UE_LIFETIMEBOUND, int32 InSize)
		: DataPtr((const CharType*)InData)
		, Size(InSize)
	{
	}

	/** Construct a view from a contiguous range of characters, such as FString or TStringBuilder. */
	template <UE::CNotCVRefTo<ViewType> CharRangeType>
		// !std::is_pointer_v is used here to force the pointer constructor to be used when passed an array or pointer.
		requires (!std::is_pointer_v<std::decay_t<CharRangeType>> && UE::CCompatibleStringViewable<CharRangeType, CharType> && !TIsTStringView_V<std::decay_t<CharRangeType>>)
	[[nodiscard]] constexpr inline TStringView(const CharRangeType& InRange UE_LIFETIMEBOUND)
		: DataPtr((const CharType*)GetDataHelper(InRange))
		, Size(IntCastChecked<int32>(GetNum(InRange)))
	{
	}

	/** Construct a view from a contiguous range of characters, such as FString or TStringBuilder. */
	template <UE::CNotCVRefTo<ViewType> CharRangeType>
	// !std::is_pointer_v is used here to force the pointer constructor to be used when passed an array or pointer.
		requires (!std::is_pointer_v<std::decay_t<CharRangeType>> && UE::CCompatibleStringViewable<CharRangeType, CharType> && TIsTStringView_V<std::decay_t<CharRangeType>>)
	[[nodiscard]] constexpr inline TStringView(const CharRangeType& InRange)
		: DataPtr((const CharType*)GetDataHelper(InRange))
		, Size(IntCastChecked<int32>(GetNum(InRange)))
	{
	}

	//////////////////////////////////
	// Start - assignment operators //
	//////////////////////////////////

	/**
	 * We need to explicitly implement assignment operators, even though they're equivalent to copying an implicitly-
	 * constructed temporary, because UE_LIFETIMEBOUND isn't transitive across implicit construction.
	 */

	/** Construct a view of the null-terminated string pointed to by InData. */
	UE_REWRITE constexpr TStringView& operator=(const CharType* InData UE_LIFETIMEBOUND)
	{
		return *this = TStringView(InData);
	}

	/** Construct a view of the null-terminated string pointed to by InData. */
	template <UE::CCompatibleCharType<CharType> OtherCharType>
	UE_REWRITE constexpr TStringView& operator=(const OtherCharType* InData UE_LIFETIMEBOUND)
	{
		return *this = TStringView(InData);
	}

	/** Construct a view from a contiguous range of characters, such as FString or TStringBuilder. */
	template <UE::CNotCVRefTo<ViewType> CharRangeType>
		// !std::is_pointer_v is used here to force the pointer constructor to be used when passed an array or pointer.
		requires (!std::is_pointer_v<std::decay_t<CharRangeType>> && UE::CCompatibleStringViewable<CharRangeType, CharType> && !TIsTStringView_V<std::decay_t<CharRangeType>>)
	UE_REWRITE constexpr TStringView& operator=(const CharRangeType& InRange UE_LIFETIMEBOUND)
	{
		return *this = TStringView(InRange);
	}

	/** Construct a view from a contiguous range of characters, such as FString or TStringBuilder. */
	template <UE::CNotCVRefTo<ViewType> CharRangeType>
	// !std::is_pointer_v is used here to force the pointer constructor to be used when passed an array or pointer.
		requires (!std::is_pointer_v<std::decay_t<CharRangeType>> && UE::CCompatibleStringViewable<CharRangeType, CharType> && TIsTStringView_V<std::decay_t<CharRangeType>>)
	UE_REWRITE constexpr TStringView& operator=(const CharRangeType& InRange)
	{
		return *this = TStringView(InRange);
	}

	////////////////////////////////
	// End - assignment operators //
	////////////////////////////////

	/** Access the character at the given index in the view. */
	[[nodiscard]] inline const CharType& operator[](int32 Index) const;

	/** Returns a pointer to the start of the view. This is NOT guaranteed to be null-terminated! */
	[[nodiscard]] constexpr inline const CharType* GetData() const
	{
		return DataPtr;
	}

	// Capacity

	/** @returns Number of bytes used for the characters in the string view */
	[[nodiscard]] constexpr inline SIZE_T NumBytes() const
	{
		return static_cast<SIZE_T>(Size) * sizeof(CharType);
	}

	/** Returns the length of the string view. */
	[[nodiscard]] constexpr inline int32 Len() const
	{
		return Size;
	}

	/** Returns whether the string view is empty. */
	[[nodiscard]] constexpr inline bool IsEmpty() const
	{
		return Size == 0;
	}

	// Modifiers

	/** Modifies the view to remove the given number of characters from the start. */
	inline void RemovePrefix(int32 CharCount)
	{
		DataPtr += CharCount;
		Size -= CharCount;
	}
	/** Modifies the view to remove the given number of characters from the end. */
	inline void RemoveSuffix(int32 CharCount)
	{
		Size -= CharCount;
	}
	/** Resets to an empty view */
	inline void	 Reset()
	{
		DataPtr = nullptr;
		Size = 0;
	}

	// Operations

	/**
	 * Copy characters from the view into a destination buffer without null termination.
	 *
	 * @param Dest Buffer to write into. Must have space for at least CharCount characters.
	 * @param CharCount The maximum number of characters to copy.
	 * @param Position The offset into the view from which to start copying.
	 *
	 * @return The number of characters written to the destination buffer.
	 */
	inline int32 CopyString(CharType* Dest, int32 CharCount, int32 Position = 0) const;

	/** Alias for Mid. */
	[[nodiscard]] inline ViewType SubStr(int32 Position, int32 CharCount) const
	{
		return Mid(Position, CharCount);
	}

	/** Returns the left-most part of the view by taking the given number of characters from the left. */
	[[nodiscard]] inline ViewType Left(int32 CharCount) const;

	/** Returns the left-most part of the view by chopping the given number of characters from the right. */
	[[nodiscard]] inline ViewType LeftChop(int32 CharCount) const;

	/** Returns the right-most part of the view by taking the given number of characters from the right. */
	[[nodiscard]] inline ViewType Right(int32 CharCount) const;

	/** Returns the right-most part of the view by chopping the given number of characters from the left. */
	[[nodiscard]] inline ViewType RightChop(int32 CharCount) const;

	/** Returns the middle part of the view by taking up to the given number of characters from the given position. */
	[[nodiscard]] inline ViewType Mid(int32 Position, int32 CharCount = MAX_int32) const;

	/** Returns the middle part of the view between any whitespace at the start and end. */
	[[nodiscard]] inline ViewType TrimStartAndEnd() const;

	/** Returns the right part of the view after any whitespace at the start. */
	[[nodiscard]] inline ViewType TrimStart() const;

	/** Returns the left part of the view before any whitespace at the end. */
	[[nodiscard]] inline ViewType TrimEnd() const;

	/** Modifies the view to be the given number of characters from the left. */
	inline void LeftInline(int32 CharCount)
	{
		*this = Left(CharCount);
	}

	/** Modifies the view by chopping the given number of characters from the right. */
	inline void LeftChopInline(int32 CharCount)
	{
		*this = LeftChop(CharCount);
	}

	/** Modifies the view to be the given number of characters from the right. */
	inline void RightInline(int32 CharCount)
	{
		*this = Right(CharCount);
	}

	/** Modifies the view by chopping the given number of characters from the left. */
	inline void RightChopInline(int32 CharCount)
	{
		*this = RightChop(CharCount);
	}

	/** Modifies the view to be the middle part by taking up to the given number of characters from the given position. */
	inline void MidInline(int32 Position, int32 CharCount = MAX_int32)
	{
		*this = Mid(Position, CharCount);
	}

	/** Modifies the view to be the middle part between any whitespace at the start and end. */
	inline void TrimStartAndEndInline()
	{
		*this = TrimStartAndEnd();
	}

	/** Modifies the view to be the right part after any whitespace at the start. */
	inline void TrimStartInline()
	{
		*this = TrimStart();
	}

	/** Modifies the view to be the left part before any whitespace at the end. */
	inline void TrimEndInline()
	{
		*this = TrimEnd();
	}

	// Comparison

	/**
	 * Check whether this view is equivalent to another view.
	 *
	 * @param OtherView    A string view that is comparable with the character type of this view.
	 * @param SearchCase   Whether the comparison should ignore case.
	 * @return 0 is equal, negative if this view is less, positive if this view is greater.
	 */
	template <UE::CCharType OtherCharType>
	[[nodiscard]] bool Equals(TStringView<OtherCharType> OtherView, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive) const
	{
		return Len() == OtherView.Len() && Compare(OtherView, SearchCase) == 0;
	}

	/**
	 * Check whether this view is equivalent to a character range.
	 *
	 * @param Other        A character range that is comparable with the character type of this view.
	 * @param SearchCase Whether the comparison should ignore case.
	 */
	template <UE::CStringViewable OtherRangeType>
	[[nodiscard]] bool Equals(const OtherRangeType& Other, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive) const;

	/**
	 * Check whether this view is equivalent to a string view.
	 *
	 * @param Other        A string that is comparable with the character type of this view.
	 * @param SearchCase Whether the comparison should ignore case.
	 */
	template <UE::CCharType OtherCharType>
	[[nodiscard]] inline bool Equals(const OtherCharType* Other, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive) const
	{
		return PrivateEquals(Other, SearchCase);
	}

	/**
	 * Compare this view with a character range.
	 *
	 * @param Other        A character range that is comparable with the character type of this view.
	 * @param SearchCase Whether the comparison should ignore case.
	 * @return 0 is equal, negative if this view is less, positive if this view is greater.
	 */
	template <UE::CStringViewable OtherRangeType>
	[[nodiscard]] int32 Compare(const OtherRangeType& Other, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive) const;

	/**
	 * Compare this view with a string view.
	 *
	 * @param Other        A string view that is comparable with the character type of this view.
	 * @param SearchCase   Whether the comparison should ignore case.
	 * @return 0 is equal, negative if this view is less, positive if this view is greater.
	 */
	template <UE::CCharType OtherCharType>
	[[nodiscard]] inline int32 Compare(TStringView<OtherCharType> Other, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive) const
	{
		return PrivateCompare(Other, SearchCase);
	}

	/**
	 * Compare this view with a null-terminated string.
	 *
	 * @param Other        A null-terminated string that is comparable with the character type of this view.
	 * @param SearchCase Whether the comparison should ignore case.
	 * @return 0 is equal, negative if this view is less, positive if this view is greater.
	 */
	template <UE::CCharType OtherCharType>
	[[nodiscard]] inline int32 Compare(const OtherCharType* Other, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive) const;

	/** Returns whether this view starts with the prefix character compared case-sensitively. */
	[[nodiscard]] inline bool StartsWith(CharType Prefix) const { return Size >= 1 && DataPtr[0] == Prefix; }
	/** Returns whether this view starts with the prefix with optional case sensitivity. */
	[[nodiscard]] inline bool StartsWith(ViewType Prefix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const;

	/** Returns whether this view ends with the suffix character compared case-sensitively. */
	[[nodiscard]] inline bool EndsWith(CharType Suffix) const { return Size >= 1 && DataPtr[Size-1] == Suffix; }
	/** Returns whether this view ends with the suffix with optional case sensitivity. */
	[[nodiscard]] inline bool EndsWith(ViewType Suffix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const;

	// Searching/Finding

	/**
	 * Search the view for the first occurrence of a search string.
	 *
	 * @param Search          The string to search for. Comparison is lexicographic.
	 * @param StartPosition   The character position to start searching from.
	 * @param SearchCase      Indicates whether the search is case sensitive or not
	 * @return The index of the first occurrence of the search string if found, otherwise INDEX_NONE.
	 */
	[[nodiscard]] inline int32 Find(ViewType Search, int32 StartPosition = 0, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive) const;

	/**
	 * Returns whether this view contains the specified substring.
	 *
	 * @param Search          Text to search for
	 * @param SearchCase      Indicates whether the search is case sensitive or not
	 * @return True if the view contains the search string, otherwise false.
	 */
	[[nodiscard]] inline bool Contains(ViewType Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive) const
	{
		return Find(Search, 0, SearchCase) != INDEX_NONE;
	}

	/**
	 * Search the view for the first occurrence of a character.
	 *
	 * @param Search           The character to search for. Comparison is lexicographic.
	 * @param OutIndex [out] The position at which the character was found, or INDEX_NONE if not found.
	 * @return True if the character was found in the view, otherwise false.
	 */
	inline bool FindChar(CharType Search, int32& OutIndex) const;

	/**
	 * Search the view for the last occurrence of a character.
	 *
	 * @param Search           The character to search for. Comparison is lexicographic.
	 * @param OutIndex [out] The position at which the character was found, or INDEX_NONE if not found.
	 * @return True if the character was found in the view, otherwise false.
	 */
	inline bool FindLastChar(CharType Search, int32& OutIndex) const;

	/**
	 * Tests if index is valid, i.e. greater than or equal to zero, and less than the number of characters in the string view.
	 *
	 * @param Index Index to test.
	 *
	 * @returns True if index is valid. False otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValidIndex(int32 Index) const
	{
		return Index >= 0 && Index < Len();
	}

private:
	static bool PrivateEquals(TStringView Lhs, const CharType* Rhs);
	static bool PrivateEquals(TStringView Lhs, TStringView Rhs);
	static bool PrivateLess(TStringView Lhs, TStringView Rhs);
	static bool PrivateGreater(TStringView Lhs, TStringView Rhs);

	template <UE::CCharType OtherCharType>
	inline bool PrivateEquals(const OtherCharType* Other, ESearchCase::Type SearchCase) const;

	template <UE::CCharType OtherCharType>
	inline int32 PrivateCompare(TStringView<OtherCharType> Other, ESearchCase::Type SearchCase) const;

public:
	[[nodiscard]] friend constexpr inline auto GetNum(TStringView String)
	{
		return String.Len();
	}

	/** Case insensitive string hash function. */
	[[nodiscard]] friend inline uint32 GetTypeHash(TStringView View)
	{
		// This must match the GetTypeHash behavior of FString
		return FCrc::Strihash_DEPRECATED(View.Len(), View.GetData());
	}

	[[nodiscard]] inline bool UEOpEquals(TStringView Rhs) const
	{
		return this->Equals(Rhs, ESearchCase::IgnoreCase);
	}

	[[nodiscard]] inline bool UEOpLessThan(TStringView Rhs) const
	{
		return this->Compare(Rhs, ESearchCase::IgnoreCase) < 0;
	}

	template <UE::CConvertibleTo<TStringView> CharRangeType>
	[[nodiscard]] UE_REWRITE auto UEOpEquals(CharRangeType&& Rhs) const
	{
		return TStringView::PrivateEquals(*this, ImplicitConv<TStringView>(Forward<CharRangeType>(Rhs)));
	}

	template <UE::CConvertibleTo<TStringView> CharRangeType>
	[[nodiscard]] UE_REWRITE auto UEOpLessThan(CharRangeType&& Rhs) const
	{
		return TStringView::PrivateLess(*this, ImplicitConv<TStringView>(Forward<CharRangeType>(Rhs)));
	}

	template <UE::CConvertibleTo<TStringView> CharRangeType>
	[[nodiscard]] UE_REWRITE auto UEOpGreaterThan(CharRangeType&& Rhs) const
	{
		return TStringView::PrivateGreater(*this, ImplicitConv<TStringView>(Forward<CharRangeType>(Rhs)));
	}

	[[nodiscard]] UE_REWRITE bool UEOpEquals(const CharType* Rhs) const
	{
		return TStringView::PrivateEquals(*this, Rhs);
	}

public:
	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	[[nodiscard]] constexpr inline const CharType* begin() const { return DataPtr; }
	[[nodiscard]] constexpr inline const CharType* end() const { return DataPtr + Size; }
	[[nodiscard]] constexpr inline TReversePointerIterator<const CharType> rbegin() const { return TReversePointerIterator<const CharType>(DataPtr + Size); }
	[[nodiscard]] constexpr inline TReversePointerIterator<const CharType> rend() const { return TReversePointerIterator<const CharType>(DataPtr); }

protected:
	const CharType* DataPtr = nullptr;
	int32 Size = 0;
};

template <typename CharRangeType>
TStringView(CharRangeType&& Range)
	-> TStringView<TElementType_T<CharRangeType>>;

template <typename CharPtrOrRangeType>
	requires (!std::is_pointer_v<CharPtrOrRangeType> || TIsCharType_V<std::remove_pointer_t<CharPtrOrRangeType>>)
[[nodiscard]] constexpr inline auto MakeStringView(const CharPtrOrRangeType& CharPtrOrRange UE_LIFETIMEBOUND) -> decltype(TStringView(CharPtrOrRange))
{
	return TStringView(CharPtrOrRange);
}

template <UE::CCharType CharType>
[[nodiscard]] constexpr inline auto MakeStringView(const CharType* CharPtr UE_LIFETIMEBOUND, int32 Size) -> decltype(TStringView(CharPtr, Size))
{
	return TStringView(CharPtr, Size);
}

//////////////////////////////////////////////////////////////////////////

#if PLATFORM_TCHAR_IS_UTF8CHAR
[[nodiscard]] inline FStringView operator""_PrivateSV(const ANSICHAR* String, size_t Size)
{
	return FStringView((const UTF8CHAR*)String, Size);
}
#else
[[nodiscard]] constexpr inline FStringView operator""_PrivateSV(const TCHAR* String, size_t Size)
{
	return FStringView(String, (int32)Size);
}
#endif

[[nodiscard]] constexpr inline FAnsiStringView operator""_PrivateASV(const ANSICHAR* String, size_t Size)
{
	return FAnsiStringView(String, (int32)Size);
}

[[nodiscard]] constexpr inline FWideStringView operator""_PrivateWSV(const WIDECHAR* String, size_t Size)
{
	return FWideStringView(String, (int32)Size);
}

[[nodiscard]] constexpr inline FUtf8StringView operator""_PrivateU8SV(const UTF8CHAR* String, size_t Size)
{
	return FUtf8StringView(String, (int32)Size);
}

#if PLATFORM_TCHAR_IS_UTF8CHAR
	#define TEXTVIEW(str) (str##_PrivateSV)
#else
	#define TEXTVIEW(str) TEXT(str##_PrivateSV)
#endif
#define ANSITEXTVIEW(str) (str##_PrivateASV)
#define WIDETEXTVIEW(str) UE_JOIN(WIDETEXT(str), _PrivateWSV)
#define UTF8TEXTVIEW(str) UE_JOIN(UTF8TEXT(str), _PrivateU8SV)

namespace UE::Core::Private
{
	template <typename CharType>
	void CharTextViewStaticAssert()
	{
		static_assert(sizeof(CharType) == 0, "Unsupported char type passed to CHARTEXTVIEW");
	}
}

// Expands out to ANSITEXTVIEW(x), TEXTVIEW(x), WIDETEXTVIEW(x) or UTF8TEXTVIEW(x) depending on CharType
#define CHARTEXTVIEW(CharType, x) \
	( \
		[]() -> decltype(auto) \
		{ \
			/* We expect <type_traits> to already have been included any time the CHARTEXTVIEW macro is used */ \
			using UnqualifiedCharType = std::remove_cv_t<CharType>; \
			if constexpr (std::is_same_v<UnqualifiedCharType, ANSICHAR>) \
			{ \
				return ANSITEXTVIEW(x); \
			} \
			else if constexpr (std::is_same_v<UnqualifiedCharType, TCHAR>) \
			{ \
				return TEXTVIEW(x); \
			} \
			else if constexpr (std::is_same_v<UnqualifiedCharType, WIDECHAR>) \
			{ \
				return WIDETEXTVIEW(x); \
			} \
			else if constexpr (std::is_same_v<UnqualifiedCharType, UTF8CHAR>) \
			{ \
				return UTF8TEXTVIEW(x); \
			} \
			else \
			{ \
				/* We want a compile error, so forward to a templated function because of the static_assert(false) problem */ \
				UE::Core::Private::CharTextViewStaticAssert<CharType>(); \
				return x; \
			} \
		}() \
	)

//////////////////////////////////////////////////////////////////////////

template <UE::CCharType CharType>
[[nodiscard]] inline const CharType& TStringView<CharType>::operator[](int32 Index) const
{
	checkf(Index >= 0 && Index < Size, TEXT("Index out of bounds on StringView: index %i on a view with a length of %i"), Index, Size);
	return DataPtr[Index];
}

template <UE::CCharType CharType>
inline int32 TStringView<CharType>::CopyString(CharType* Dest, int32 CharCount, int32 Position) const
{
	const int32 CopyCount = FMath::Min(Size - Position, CharCount);
	if (CopyCount > 0)
	{
		FMemory::Memcpy(Dest, DataPtr + Position, CopyCount * sizeof(CharType));
	}
	return CopyCount;
}

template <UE::CCharType CharType>
[[nodiscard]] inline TStringView<CharType> TStringView<CharType>::Left(int32 CharCount) const
{
	return ViewType(DataPtr, FMath::Clamp(CharCount, 0, Size));
}

template <UE::CCharType CharType>
[[nodiscard]] inline TStringView<CharType> TStringView<CharType>::LeftChop(int32 CharCount) const
{
	return ViewType(DataPtr, FMath::Clamp(Size - CharCount, 0, Size));
}

template <UE::CCharType CharType>
[[nodiscard]] inline TStringView<CharType> TStringView<CharType>::Right(int32 CharCount) const
{
	const int32 OutLen = FMath::Clamp(CharCount, 0, Size);
	return ViewType(DataPtr + Size - OutLen, OutLen);
}

template <UE::CCharType CharType>
[[nodiscard]] inline TStringView<CharType> TStringView<CharType>::RightChop(int32 CharCount) const
{
	const int32 OutLen = FMath::Clamp(Size - CharCount, 0, Size);
	return ViewType(DataPtr + Size - OutLen, OutLen);
}

template <UE::CCharType CharType>
[[nodiscard]] inline TStringView<CharType> TStringView<CharType>::Mid(int32 Position, int32 CharCount) const
{
	const CharType* CurrentStart = GetData();
	const int32 CurrentLength = Len();

	// Clamp minimum position at the start of the string, adjusting the length down if necessary
	const int32 NegativePositionOffset = (Position < 0) ? Position : 0;
	CharCount += NegativePositionOffset;
	Position  -= NegativePositionOffset;

	// Clamp maximum position at the end of the string
	Position = (Position > CurrentLength) ? CurrentLength : Position;

	// Clamp count between 0 and the distance to the end of the string
	CharCount = FMath::Clamp(CharCount, 0, (CurrentLength - Position));

	ViewType Result = ViewType(CurrentStart + Position, CharCount);
	return Result;
}

template <UE::CCharType CharType>
[[nodiscard]] inline TStringView<CharType> TStringView<CharType>::TrimStartAndEnd() const
{
	return TrimStart().TrimEnd();
}

template <UE::CCharType CharType>
[[nodiscard]] inline TStringView<CharType> TStringView<CharType>::TrimStart() const
{
	int32 SpaceCount = 0;
	for (CharType Char : *this)
	{
		if (!TChar<CharType>::IsWhitespace(Char))
		{
			break;
		}
		++SpaceCount;
	}
	return ViewType(DataPtr + SpaceCount, Size - SpaceCount);
}

template <UE::CCharType CharType>
[[nodiscard]] inline TStringView<CharType> TStringView<CharType>::TrimEnd() const
{
	int32 NewSize = Size;
	while (NewSize && TChar<CharType>::IsWhitespace(DataPtr[NewSize - 1]))
	{
		--NewSize;
	}
	return ViewType(DataPtr, NewSize);
}

template <UE::CCharType CharType>
template <UE::CCharType OtherCharType>
[[nodiscard]] inline bool TStringView<CharType>::PrivateEquals(const OtherCharType* const Other, ESearchCase::Type SearchCase) const
{
	if (IsEmpty())
	{
		return !*Other;
	}

	if (SearchCase == ESearchCase::CaseSensitive)
	{
		return FPlatformString::Strncmp(GetData(), Other, Len()) == 0 && !Other[Len()];
	}
	else
	{
		return FPlatformString::Strnicmp(GetData(), Other, Len()) == 0 && !Other[Len()];
	}
}

template <UE::CCharType CharType>
template <UE::CCharType OtherCharType>
[[nodiscard]] inline int32 TStringView<CharType>::PrivateCompare(TStringView<OtherCharType> OtherView, ESearchCase::Type SearchCase) const
{
	const int32 SelfLen = Len();
	const int32 OtherLen = OtherView.Len();
	const int32 MinLen = FMath::Min(SelfLen, OtherLen);

	if (!SelfLen || !OtherLen)
	{
		return SelfLen - OtherLen;
	}

	int Result;
	if (SearchCase == ESearchCase::CaseSensitive)
	{
		Result = FPlatformString::Strncmp(GetData(), OtherView.GetData(), MinLen);
	}
	else
	{
		Result = FPlatformString::Strnicmp(GetData(), OtherView.GetData(), MinLen);
	}

	if (Result != 0)
	{
		return Result;
	}

	return SelfLen - OtherLen;
}

template <UE::CCharType CharType>
template <UE::CCharType OtherCharType>
[[nodiscard]] inline int32 TStringView<CharType>::Compare(const OtherCharType* const Other, ESearchCase::Type SearchCase) const
{
	if (IsEmpty())
	{
		return -!!*Other;
	}

	int Result;
	if (SearchCase == ESearchCase::CaseSensitive)
	{
		Result = FPlatformString::Strncmp(GetData(), Other, Len());
	}
	else
	{
		Result = FPlatformString::Strnicmp(GetData(), Other, Len());
	}

	if (Result != 0)
	{
		return Result;
	}

	// Equal if Other[Len()] is '\0' and less otherwise. !!Other[Len()] is 0 for '\0' and 1 otherwise.
	return -!!Other[Len()];
}

template <UE::CCharType CharType>
[[nodiscard]] inline bool TStringView<CharType>::StartsWith(ViewType Prefix, ESearchCase::Type SearchCase) const
{
	return Prefix.Equals(Left(Prefix.Len()), SearchCase);
}

template <UE::CCharType CharType>
[[nodiscard]] inline bool TStringView<CharType>::EndsWith(ViewType Suffix, ESearchCase::Type SearchCase) const
{
	return Suffix.Equals(Right(Suffix.Len()), SearchCase);
}

template <UE::CCharType CharType>
[[nodiscard]] inline int32 TStringView<CharType>::Find(const ViewType Search, const int32 StartPosition, ESearchCase::Type SearchCase) const
{
	const int32 Index = UE::String::FindFirst(RightChop(StartPosition), Search, SearchCase);
	return Index == INDEX_NONE ? INDEX_NONE : Index + StartPosition;
}

template <UE::CCharType CharType>
inline bool TStringView<CharType>::FindChar(const CharType Search, int32& OutIndex) const
{
	OutIndex = UE::String::FindFirstChar(*this, Search);
	return OutIndex != INDEX_NONE;
}

template <UE::CCharType CharType>
inline bool TStringView<CharType>::FindLastChar(const CharType Search, int32& OutIndex) const
{
	OutIndex = UE::String::FindLastChar(*this, Search);
	return OutIndex != INDEX_NONE;
}

template <UE::CCharType CharType>
[[nodiscard]] inline bool TStringView<CharType>::PrivateEquals(TStringView Lhs, const CharType* Rhs)
{
	return Lhs.Equals(Rhs, ESearchCase::IgnoreCase);
}

template <UE::CCharType CharType>
[[nodiscard]] inline bool TStringView<CharType>::PrivateEquals(TStringView Lhs, TStringView Rhs)
{
	return Lhs.Equals(Rhs, ESearchCase::IgnoreCase);
}

template <UE::CCharType CharType>
[[nodiscard]] inline bool TStringView<CharType>::PrivateLess(TStringView Lhs, TStringView Rhs)
{
	return Lhs.Compare(Rhs, ESearchCase::IgnoreCase) < 0;
}

template <UE::CCharType CharType>
[[nodiscard]] inline bool TStringView<CharType>::PrivateGreater(TStringView Lhs, TStringView Rhs)
{
	return Lhs.Compare(Rhs, ESearchCase::IgnoreCase) > 0;
}

template <UE::CCharType CharType>
template <UE::CStringViewable OtherRangeType>
[[nodiscard]] inline bool TStringView<CharType>::Equals(const OtherRangeType& Other, ESearchCase::Type SearchCase) const
{
	return Equals(MakeStringView(Other), SearchCase);
}

template <UE::CCharType CharType>
template <UE::CStringViewable OtherRangeType>
[[nodiscard]] inline int32 TStringView<CharType>::Compare(const OtherRangeType& Other, ESearchCase::Type SearchCase) const
{
	return Compare(MakeStringView(Other), SearchCase);
}

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Templates/Requires.h"
#endif
