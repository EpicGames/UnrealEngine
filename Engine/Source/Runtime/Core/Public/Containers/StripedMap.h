// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Map.h"
#include "Async/SharedLock.h"
#include "Async/SharedMutex.h"
#include "Async/UniqueLock.h"
#include "HAL/PlatformTLS.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Misc/ScopeRWLock.h"

/**
 * This locking policy uses FSharedMutex which is lightweight and doesn't consume any OS resources.
 **/
struct FSharedMutexStripedMapLockingPolicy
{
	typedef UE::FSharedMutex                  MutexType;
	typedef UE::TUniqueLock<UE::FSharedMutex> ExclusiveLockType;
	typedef UE::TSharedLock<UE::FSharedMutex> SharedLockType;
};

/**
 * This locking policy uses a FTransactionallySafeRWLock which supports AutoRTFM but is backed by a RWLock which consumes OS resources.
 **/
struct FTransactionallySafeStripedMapLockingPolicy
{
	typedef FTransactionallySafeRWLock        MutexType;
	typedef UE::TWriteScopeLock<MutexType>    ExclusiveLockType;
	typedef UE::TReadScopeLock<MutexType>     SharedLockType;
};

// Use FTransactionallySafeRWLock by default for now because there is no transactionally-safe
// FSharedMutex and a bug in FSharedMutex is currently causing deadlocks.
typedef FTransactionallySafeStripedMapLockingPolicy FDefaultStripedMapLockingPolicy;

/**
 * The base class of striped maps which is a wrapper that adds thread-safety and contention reduction over regular maps.
 * 
 * The interface is slightly modified compared to regular maps to avoid some thread-safety issues that would arise if we 
 * returned pointers or reference to memory inside the map after the lock on a bucket had been released.
 * 
 * The ByHash() functions are somewhat dangerous but particularly useful in two scenarios:
 * -- Heterogeneous lookup to avoid creating expensive keys like FString when looking up by const TCHAR*.
 *	  You must ensure the hash is calculated in the same way as ElementType is hashed.
 *    If possible put both ComparableKey and ElementType hash functions next to each other in the same header
 *    to avoid bugs when the ElementType hash function is changed.
 * -- Reducing contention around hash tables protected by a lock.
 *    This class manage this automatically so you don't have to work with ByHash function in this case.
 * 
 **/
template <int32 BucketCount, typename BaseMapType, typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs, typename LockingPolicy>
class TStripedMapBase
{
	static_assert(BucketCount > 0, "The BucketCount needs to be at least 1");
public:
	typedef BaseMapType                       MapType;
	typedef typename BaseMapType::KeyInitType KeyInitType;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "KeyConstPointerType is deprecated - use KeyInitType instead.")
	typedef typename BaseMapType::KeyConstPointerType KeyConstPointerType;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	typedef KeyFuncs                          KeyFuncsType;

protected:
	struct FDebuggableMutex
	{
		typename LockingPolicy::MutexType Mutex;
		uint32 ExclusiveLockOwnerThreadId   = 0;
		std::atomic<uint32> SharedLockCount = 0;
	};

	struct FDebuggableSharedLock
	{
		FDebuggableMutex& DebuggableMutex;
		typename LockingPolicy::SharedLockType InnerLock;
		FDebuggableSharedLock(FDebuggableMutex& InDebuggableMutex)
			: DebuggableMutex(InDebuggableMutex)
			, InnerLock(InDebuggableMutex.Mutex)
		{
			check(DebuggableMutex.ExclusiveLockOwnerThreadId == 0);
			DebuggableMutex.SharedLockCount++;
		}

		~FDebuggableSharedLock()
		{
			DebuggableMutex.SharedLockCount--;
		}
	};

	struct FDebuggableExclusiveLock
	{
		FDebuggableMutex& DebuggableMutex;
		typename LockingPolicy::ExclusiveLockType InnerLock;
		FDebuggableExclusiveLock(FDebuggableMutex& InDebuggableMutex)
			: DebuggableMutex(InDebuggableMutex)
			, InnerLock(InDebuggableMutex.Mutex)
		{
			check(DebuggableMutex.ExclusiveLockOwnerThreadId == 0);
			DebuggableMutex.ExclusiveLockOwnerThreadId = FPlatformTLS::GetCurrentThreadId();
		}

		~FDebuggableExclusiveLock()
		{
			DebuggableMutex.ExclusiveLockOwnerThreadId = 0;
		}
	};

#define WITH_STRIPEDMAP_DEBUGGABLE_MUTEX 0
#if WITH_STRIPEDMAP_DEBUGGABLE_MUTEX
	typedef FDebuggableMutex                          MutexType;
	typedef FDebuggableExclusiveLock                  ExclusiveLockType;
	typedef FDebuggableSharedLock                     SharedLockType;
#else
	typedef typename LockingPolicy::MutexType         MutexType;
	typedef typename LockingPolicy::ExclusiveLockType ExclusiveLockType;
	typedef typename LockingPolicy::SharedLockType    SharedLockType;
#endif
#undef WITH_STRIPEDMAP_DEBUGGABLE_MUTEX

	struct FBucket
	{
		mutable MutexType Lock;
		MapType           Map;
	} Buckets[BucketCount];

	uint32 GetBucketIndex(uint32 Hash) const
	{
		if constexpr (BucketCount == 1)
		{
			return 0;
		}
		else
		{
			return Hash % BucketCount;
		}
	}

	template <typename FunctionType>
	decltype(auto) ApplyUnlockedByHash(uint32 InHash, FunctionType&& InFunction)
	{
		return InFunction(InHash, Buckets[GetBucketIndex(InHash)]);
	}

	template <typename FunctionType>
	decltype(auto) ApplyUnlocked(KeyInitType InKey, FunctionType&& InFunction)
	{
		return ApplyUnlockedByHash(KeyFuncsType::GetKeyHash(InKey), Forward<FunctionType>(InFunction));
	}

	template <typename LockType, typename FunctionType>
	decltype(auto) ApplyByHash(uint32 InHash, FunctionType&& InFunction) const
	{
		const FBucket& Bucket = Buckets[GetBucketIndex(InHash)];

		LockType ScopeLock(Bucket.Lock);
		return InFunction(InHash, Bucket.Map);
	}

	template <typename LockType, typename FunctionType>
	decltype(auto) Apply(KeyInitType InKey, FunctionType&& InFunction) const
	{
		return ApplyByHash<LockType>(KeyFuncsType::GetKeyHash(InKey), Forward<FunctionType>(InFunction));
	}

	template <typename LockType, typename FunctionType>
	decltype(auto) ApplyByHash(uint32 InHash, FunctionType&& InFunction)
	{
		FBucket& Bucket = Buckets[GetBucketIndex(InHash)];

		LockType ScopeLock(Bucket.Lock);
		return InFunction(InHash, Bucket.Map);
	}

	template <typename LockType, typename FunctionType>
	decltype(auto) Apply(KeyInitType InKey, FunctionType&& InFunction)
	{
		return ApplyByHash<LockType>(KeyFuncsType::GetKeyHash(InKey), Forward<FunctionType>(InFunction));
	}

	template <typename FunctionType>
	decltype(auto) Write(KeyInitType InKey, FunctionType&& InFunction)
	{
		return Apply<ExclusiveLockType>(InKey, Forward<FunctionType>(InFunction));
	}

	template <typename FunctionType>
	decltype(auto) WriteByHash(uint32 InHash, KeyInitType InKey, FunctionType&& InFunction)
	{
		return ApplyByHash<ExclusiveLockType>(InHash, InKey, Forward<FunctionType>(InFunction));
	}

	template <typename FunctionType>
	decltype(auto) Read(KeyInitType InKey, FunctionType&& InFunction)
	{
		return Apply<SharedLockType>(InKey, Forward<FunctionType>(InFunction));
	}

	template <typename FunctionType>
	decltype(auto) ReadByHash(uint32 InHash, KeyInitType InKey, FunctionType&& InFunction)
	{
		return ApplyByHash<SharedLockType>(InHash, InKey, Forward<FunctionType>(InFunction));
	}

	template <typename FunctionType>
	decltype(auto) Read(KeyInitType InKey, FunctionType&& InFunction) const
	{
		return Apply<SharedLockType>(InKey, Forward<FunctionType>(InFunction));
	}

	template <typename FunctionType>
	decltype(auto) ReadByHash(uint32 InHash, KeyInitType InKey, FunctionType&& InFunction) const
	{
		return ApplyByHash<SharedLockType>(InHash, InKey, Forward<FunctionType>(InFunction));
	}

	template <typename FunctionType>
	void ForEachMap(FunctionType&& InFunction)
	{
		for (FBucket& Bucket : Buckets)
		{
			ExclusiveLockType ScopeLock(Bucket.Lock);
			InFunction(Bucket.Map);
		}
	}

	template <typename FunctionType>
	void ParallelForEachMap(FunctionType&& InFunction)
	{
		ParallelFor(
			BucketCount,
			[this, &InFunction](int32 BucketIndex)
			{
				FBucket& Bucket = Buckets[BucketIndex];
				ExclusiveLockType ScopeLock(Bucket.Lock);
				InFunction(Bucket.Map);
			}
		);
	}

	template <typename FunctionType>
	void ForEachMap(FunctionType&& InFunction) const
	{
		for (const FBucket& Bucket : Buckets)
		{
			SharedLockType ScopeLock(Bucket.Lock);
			InFunction(Bucket.Map);
		}
	}

	template <typename FunctionType>
	void ParallelForEachMap(FunctionType&& InFunction) const
	{
		ParallelFor(
			BucketCount,
			[this, &InFunction](int32 BucketIndex)
			{
				const FBucket& Bucket = Buckets[BucketIndex];
				SharedLockType ScopeLock(Bucket.Lock);
				InFunction(Bucket.Map);
			}
		);
	}

public:
	/**
	 * Gets you a copy of the value.
	 *
	 *   Best for simple value types like PODs or TSharedPtr.
	 *
	 * @param InKey The key to look for.
	 * @return a copy of the value for this key.
	 */
	ValueType FindRef(KeyInitType InKey) const
	{
		return Read(InKey, [&InKey](uint32 Hash, const MapType& Map)
			{
				const ValueType* Result = Map.FindByHash(Hash, InKey);
				if (Result)
				{
					return *Result;
				}
				
				return ValueType();
			}
		);
	}

	/**
	 * Calls a function when the value is found while holding a lock on the map.
	 * 
	 *   Best for more complex types that you don't want to wrap under TSharedPtr
	 *   and where returning a copy would be wasteful.
	 * 
	 * @param InKey       The key to look for.
	 * @param InFunction  The function to call on the value (if found).
	 * @return true if the map contains the key and a value was found.
	 */
	template <typename FunctionType>
	bool FindAndApply(KeyInitType InKey, FunctionType&& InFunction) const
	{
		return Read(InKey, [&InKey, &InFunction](uint32 Hash, const MapType& Map)
			{
				const ValueType* Result = Map.FindByHash(Hash, InKey);
				if (Result)
				{
					InFunction(*Result);
					return true;
				}

				return false;
			}
		);
	}

	/**
	 * Calls a function to update a value if it has been found.
	 *
	 * @param InKey             The key to check for.
	 * @param InUpdateFunction  The function to call on the value (if found).
	 * @return true if the map contains the key and a value was found.
	 */
	template <typename UpdateFunctionType>
	bool FindAndApply(KeyInitType InKey, UpdateFunctionType&& InUpdateFunction)
	{
		return Write(InKey, [&InKey, &InUpdateFunction](uint32 Hash, MapType& Map)
			{
				ValueType* Result = Map.FindByHash(Hash, InKey);
				if (Result)
				{
					InUpdateFunction(*Result);
					return true;
				}

				return false;
			}
		);
	}

	/**
	 * Check if map contains the specified key.
	 *
	 * @param InKey The key to check for.
	 * @return true if the map contains the key.
	 */
	[[nodiscard]] bool Contains(KeyInitType InKey) const
	{
		return Read(InKey, [&InKey](uint32 Hash, const MapType& Map) { return Map.ContainsByHash(Hash, InKey); });
	}

	/**
	 * Sets the value associated with a key.
	 *
	 * @param InKey The key to associate the value with.
	 * @param InValue The value to associate with the key.
	 */
	template <typename InitKeyType = KeyType, typename InitValueType = ValueType>
	void Emplace(InitKeyType&& InKey, InitValueType&& InValue)
	{
		return Write(InKey, [&InKey, &InValue](uint32 Hash, MapType& Map) 
			{
				Map.EmplaceByHash(Hash, Forward<InitKeyType>(InKey), Forward<InitValueType>(InValue));
			}
		);
	}

	/**
	 * Set the value associated with a key.
	 *
	 * @param InKey The key to associate the value with.
	 * @param InValue The value to associate with the key.
	 */
	void Add(const KeyType& InKey, const ValueType& InValue)
	{
		return Emplace(InKey, InValue);
	}													   
				
	/**
	 * Set the value associated with a key.
	 *
	 * @param InKey The key to associate the value with.
	 * @param InValue The value to associate with the key.
	 */
	void Add(const KeyType& InKey, ValueType&& InValue)
	{													   
		return Emplace(InKey, MoveTempIfPossible(InValue));
	}													   
	
	/**
	 * Set the value associated with a key.
	 *
	 * @param InKey The key to associate the value with.
	 * @param InValue The value to associate with the key.
	 */
	void Add(KeyType&& InKey, const ValueType& InValue)
	{													   
		return Emplace(MoveTempIfPossible(InKey), InValue);
	}

	/**
	 * Set the value associated with a key.
	 *
	 * @param InKey The key to associate the value with.
	 * @param InValue The value to associate with the key.
	 */
	void Add(KeyType&& InKey, ValueType&& InValue)
	{													   
		return Emplace(MoveTempIfPossible(InKey), MoveTempIfPossible(InValue));
	}

	/**
	* Finds or produce a value associated with the key.
	*
	* @param InKey             The key to look for.
	* @param InProduceFunction The function to call to produce a new value if the key is missing.
	* @return a copy the value associated with the key.
	*/
	template <typename ProduceFunctionType>
	ValueType FindOrProduce(const KeyType& InKey, ProduceFunctionType&& InProduceFunction)
	{
		return ApplyUnlocked(InKey,
			[this, &InKey, &InProduceFunction](uint32 Hash, FBucket& Bucket)
			{
				{
					SharedLockType ScopeLock(Bucket.Lock);
					const ValueType* Result = Bucket.Map.FindByHash(Hash, InKey);
					if (Result)
					{
						return *Result;
					}
				}

				ExclusiveLockType ScopeLock(Bucket.Lock);
				ValueType* Result = Bucket.Map.FindByHash(Hash, InKey);
				if (Result)
				{
					return *Result;
				}

				return Bucket.Map.AddByHash(Hash, InKey, InProduceFunction());
			});
	}

	/**
	* Calls ProduceFunction to produce a value if the key is missing, then calls ApplyFunction on the value.
	*
	* @param InKey             The key to look for.
	* @param InProduceFunction The function to call to produce a new value associated with the key.
	* @param InApplyFunction   The function to call with the const value reference when the key is found or has been produced.
	*/
	template <typename ProduceFunctionType, typename ApplyFunctionType>
	void FindOrProduceAndApply(const KeyType& InKey, ProduceFunctionType&& InProduceFunction, ApplyFunctionType&& InApplyFunction)
	{
		ApplyUnlocked(InKey,
			[this, &InKey, &InApplyFunction, &InProduceFunction](uint32 Hash, FBucket& Bucket)
			{
				{
					SharedLockType ScopeLock(Bucket.Lock);
					const ValueType* Result = Bucket.Map.FindByHash(Hash, InKey);
					if (Result)
					{
						InApplyFunction(*Result);
						return;
					}
				}

				ExclusiveLockType ScopeLock(Bucket.Lock);
				ValueType* Result = Bucket.Map.FindByHash(Hash, InKey);
				if (Result)
				{
					InApplyFunction(*Result);
					return;
				}

				InApplyFunction(Bucket.Map.AddByHash(Hash, InKey, InProduceFunction()));
			});
	}

	/**
	* Calls TryProduceFunction to produce a value if the key is missing, then calls ApplyFunction on the value if one exists.
	*
	* @param InKey                The key to look for.
	* @param InTryProduceFunction The function to call to produce a new value associated with the key.
	* @param InApplyFunction      The function to call with the const value reference when the key is found or has been produced.
	* @return true if a value was found or produced, false if TryProduceFunction failed.
	*/
	template <typename TryProduceFunctionType, typename ApplyFunctionType>
	bool FindOrTryProduceAndApply(const KeyType& InKey, TryProduceFunctionType&& InTryProduceFunction, ApplyFunctionType&& InApplyFunction)
	{
		return ApplyUnlocked(InKey,
			[this, &InKey, &InApplyFunction, &InTryProduceFunction](uint32 Hash, FBucket& Bucket)
			{
				{
					SharedLockType ScopeLock(Bucket.Lock);
					const ValueType* Result = Bucket.Map.FindByHash(Hash, InKey);
					if (Result)
					{
						InApplyFunction(*Result);
						return true;
					}
				}

				ExclusiveLockType ScopeLock(Bucket.Lock);
				const ValueType* Result = Bucket.Map.FindByHash(Hash, InKey);
				if (Result)
				{
					InApplyFunction(*Result);
					return true;
				}

				ValueType Value;
				if (InTryProduceFunction(Value))
				{
					InApplyFunction(Bucket.Map.AddByHash(Hash, InKey, MoveTemp(Value)));
					return true;
				}

				return false;
			});
	}

	/**
	* Calls ProduceFunction to produce a value if the key is missing, then calls ApplyFunction on the value.
	*
	* @param InKey             The key to look for.
	* @param InProduceFunction The function to call to produce a new value associated with the key.
	* @param InApplyFunction   The function to call with the value reference when the key is found or has been produced.
	*/
	template <typename ProduceFunctionType, typename ApplyFunctionType>
	void FindOrProduceAndApplyForWrite(const KeyType& InKey, ProduceFunctionType&& InProduceFunction, ApplyFunctionType&& InApplyFunction)
	{
		return Write(InKey, [&InKey, &InProduceFunction, &InApplyFunction](uint32 Hash, MapType& Map)
		{
			ValueType* Result = Map.FindByHash(Hash, InKey);
			if (Result)
			{
				InApplyFunction(*Result);
			}
			else
			{
				InApplyFunction(Map.EmplaceByHash(Hash, InKey, InProduceFunction()));
			}
		});
	}

	/**
	* Calls TryProduceFunction to produce a value if the key is missing, then calls ApplyFunction on the value if one exists.
	*
	* @param InKey                The key to look for.
	* @param InTryProduceFunction The function to call to produce a new value associated with the key.
	* @param InApplyFunction      The function to call with the value reference when the key is found or has been produced.
	* @return true if a value was found or produced, false if TryProduceFunction failed.
	*/
	template <typename TryProduceFunctionType, typename ApplyFunctionType>
	bool FindOrTryProduceAndApplyForWrite(const KeyType& InKey, TryProduceFunctionType&& InTryProduceFunction, ApplyFunctionType&& InApplyFunction)
	{
		return Write(InKey, [&InKey, &InTryProduceFunction, &InApplyFunction](uint32 Hash, MapType& Map)
		{
			ValueType* Result = Map.FindByHash(Hash, InKey);
			if (Result)
			{
				InApplyFunction(*Result);
				return true;
			}

			ValueType Value;
			if (InTryProduceFunction(Value))
			{
				InApplyFunction(Map.AddByHash(Hash, InKey, MoveTemp(Value)));
				return true;
			}

			return false;
		});
	}

	/**
	 * Remove all value associations for a key.
	 *
	 * @param InKey The key to remove associated values for.
	 * @return The number of values that were associated with the key.
	 */
	int32 Remove(KeyInitType InKey)
	{
		return Write(InKey, [&InKey](uint32 Hash, MapType& Map) { return Map.RemoveByHash(Hash, InKey); });
	}
	
	/** See Remove() and class documentation section on ByHash() functions */
	int32 RemoveByHash(uint32 InKeyHash, KeyInitType InKey)
	{
		return WriteByHash(InKeyHash, InKey, [&InKey](uint32 Hash, MapType& Map) { return Map.RemoveByHash(Hash, InKey); });
	}

	/**
	 * Removes only the element associated by the key if the predicate returns true.
	 *
	 * @param InKey        The key to remove associated values for.
	 * @param InPredicate  The predicate to call to determine if a value should be removed.
	 * @return The number of values that were removed.
	 */
	template <typename PredicateType>
	int32 RemoveIf(KeyInitType InKey, PredicateType&& InPredicate)
	{
		return Write(InKey,
			[&InKey, &InPredicate](uint32 Hash, MapType& Map)
			{
				FSetElementId ElementId = Map.FindIdByHash(Hash, InKey);
				if (ElementId.IsValidId() && InPredicate(Map.Get(ElementId).Value))
				{
					Map.Remove(ElementId);
					return 1;
				}

				return 0;
			});
	}

	/**
	 * Removes all elements where the predicate returns true.
	 *
	 * @param InPredicate  The predicate to call to determine if a pair should be removed.
	 * @return The number of pairs that were removed.
	 */
	template <typename PredicateType>
	int32 RemoveIf(PredicateType&& InPredicate)
	{
		int32 RemovedCount = 0;
		ForEachMap(
			[&RemovedCount, &InPredicate](MapType& Map)
			{
				for (auto It = Map.CreateIterator(); It; ++It)
				{
					if (InPredicate(*It))
					{
						It.RemoveCurrent();
						RemovedCount++;
					}
				}
			}
		);
		return RemovedCount;
	}

	/**
	 * Removes all elements where the predicate returns true.
	 *
	 * @param InPredicate  A thread-safe predicate to call to determine if a pair should be removed.
	 * @return The number of pairs that were removed.
	 */
	template <typename PredicateType>
	int32 ParallelRemoveIf(PredicateType&& InPredicate)
	{
		std::atomic<int32> RemovedCount = 0;
		ParallelForEachMap(
			[&RemovedCount, &InPredicate](MapType& Map)
			{
				int32 LocalRemoveCount = 0;
				for (auto It = Map.CreateIterator(); It; ++It)
				{
					if (InPredicate(*It))
					{
						It.RemoveCurrent();
						LocalRemoveCount++;
					}
				}
				RemovedCount += LocalRemoveCount;
			}
		);
		return RemovedCount.load();
	}

	/**
	 * Remove the pair with the specified key and copies the value
	 * that was removed to the ref parameter
	 *
	 * @param InKey The key to search for
	 * @param OutRemovedValue If found, the value that was removed (not modified if the key was not found)
	 * @return whether or not the key was found
	 */
	bool RemoveAndCopyValue(KeyInitType InKey, ValueType& OutRemovedValue)
	{
		return Write(InKey, [&InKey, &OutRemovedValue](uint32 Hash, MapType& Map) 
			{
				return Map.RemoveAndCopyValueByHash(Hash, InKey, OutRemovedValue); 
			});
	}

	/**
	 * Find a pair with the specified key, removes it from the map, and returns the value part of the pair.
	 *
	 * If no pair was found, an exception is thrown.
	 *
	 * @param InKey The key to search for
	 * @return      The value that was removed
	 */
	ValueType FindAndRemoveChecked(KeyInitType InKey)
	{
		return Write(InKey, [&InKey](uint32 Hash, MapType& Map)
			{
				return Map.FindAndRemoveChecked(InKey); 
			}
		);
	}

	/** Removes all elements from the map. */
	void Empty()
	{
		ForEachMap(
			[](MapType& Map)
			{
				Map.Empty();
			}
		);
	}

	/** Efficiently empties out the map but preserves all allocations and capacities */
	void Reset()
	{
		ForEachMap(
			[](MapType& Map)
			{
				Map.Reset();
			}
		);
	}

	/** Shrinks the pair set to avoid slack. */
	void Shrink()
	{
		ForEachMap(
			[](MapType& Map)
			{
				Map.Shrink();
			}
		);
	}

	/** Compacts the pair set to remove holes */
	void Compact()
	{
		ForEachMap(
			[](MapType& Map)
			{
				Map.Compact();
			}
		);
	}

	/** @return The number of elements in the map. */
	[[nodiscard]] int32 Num() const
	{
		int32 Count = 0;
		ForEachMap(
			[&Count](const MapType& Map)
			{
				Count += Map.Num();
			}
		);

		return Count;
	}

	/**
	 * Helper function to return the amount of memory allocated by this
	 * container.
	 * Only returns the size of allocations made directly by the container, not the elements themselves.
	 *
	 * @returns Number of bytes allocated by this container.
	 */
	[[nodiscard]] SIZE_T GetAllocatedSize() const
	{
		SIZE_T Total = 0;
		ForEachMap(
			[&Total](const MapType& Map)
			{
				Total += Map.GetAllocatedSize();
			}
		);

		return Total;
	}

	/**
	 * Reserves memory such that the map can contain at least Number elements. The reserved memory is distributed across stripes.
	 *
	 * @param Number The number of elements that the map should be able to contain after allocation.
	 * @see Shrink
	 */
	void Reserve(int32 Number)
	{
		int32 NumberPerMap = Number / BucketCount;
		ForEachMap(
			[NumberPerMap](MapType& Map)
			{
				Map.Reserve(NumberPerMap);
			}
		);
	}

	/**
	 * Calls a function on all elements of the map with exclusive access (elements can be modified).
	 *
	 * @param InFunction The callback to run on each Pair of the map.
	 */
	template <typename FunctionType>
	void ForEach(FunctionType&& InFunction)
	{
		ForEachMap(
			[&InFunction](MapType& Map)
			{
				for (auto& Item : Map)
				{
					InFunction(Item);
				}
			}
		);
	}

	/**
	 * Calls a function on all elements of the map with exclusive access (elements can be modified).
	 *
	 * @param InFunction A thread-safe callback to run on each Pair of the map.
	 */
	template <typename FunctionType>
	void ParallelForEach(FunctionType&& InFunction)
	{
		ParallelForEachMap(
			[&InFunction](MapType& Map)
			{
				for (auto& Item : Map)
				{
					InFunction(Item);
				}
			}
		);
	}

	/**
	 * Calls a function on all elements of the map with shared access (elements can only be read).
	 *
	 * @param InFunction The callback to run on each Pair of the map.
	 */
	template <typename FunctionType>
	void ForEach(FunctionType&& InFunction) const
	{
		ForEachMap(
			[&InFunction](const MapType& Map)
			{
				for (const auto& Item : Map)
				{
					InFunction(Item);
				}
			}
		);
	}

	/**
	 * Calls a function on all elements of the map with shared access (elements can only be read).
	 *
	 * @param InFunction A thread-safe callback to run on each Pair of the map.
	 */
	template <typename FunctionType>
	void ParallelForEach(FunctionType&& InFunction) const
	{
		ParallelForEachMap(
			[&InFunction](const MapType& Map)
			{
				for (const auto& Item : Map)
				{
					InFunction(Item);
				}
			}
		);
	}

	/**
	 * Get the unique keys contained within this map.
	 *
	 * @param OutKeys Upon return, contains the set of unique keys in this map.
	 * @return The number of unique keys in the map.
	 */
	template<typename Allocator>
	int32 GetKeys(TArray<KeyType, Allocator>& OutKeys) const
	{
		OutKeys.Reset();
		TSet<KeyType> VisitedKeys;
		ForEach(
			[&OutKeys, &VisitedKeys](const auto& Pair)
			{
				if (!VisitedKeys.Contains(Pair.Key))
				{
					OutKeys.Add(Pair.Key);
					VisitedKeys.Add(Pair.Key);
				}
			}
		);
		return OutKeys.Num();
	}
};

/**
 * A wrapper over a TMap class with additional thread-safety guarantees and contention reduction.
 **/
template <
	int32 BucketCount, 
	typename KeyType, 
	typename ValueType, 
	typename SetAllocator = FDefaultSetAllocator, 
	typename KeyFuncs = TDefaultMapHashableKeyFuncs<KeyType, ValueType, false>, 
	typename LockingPolicy = FDefaultStripedMapLockingPolicy
>
class TStripedMap : public TStripedMapBase<BucketCount, TMap<KeyType, ValueType, SetAllocator, KeyFuncs>, KeyType, ValueType, SetAllocator, KeyFuncs, LockingPolicy>
{
public:
	using Super = TStripedMapBase<BucketCount, TMap<KeyType, ValueType, SetAllocator, KeyFuncs>, KeyType, ValueType, SetAllocator, KeyFuncs, LockingPolicy>;
};

/**
 * A wrapper over a TMultiMap class with additional thread-safety guarantees and contention reduction.
 **/
template <
	int32 BucketCount, 
	typename KeyType, 
	typename ValueType, 
	typename SetAllocator = FDefaultSetAllocator, 
	typename KeyFuncs = TDefaultMapHashableKeyFuncs<KeyType, ValueType, true>, 
	typename LockingPolicy = FDefaultStripedMapLockingPolicy
>
class TStripedMultiMap : public TStripedMapBase<BucketCount, TMultiMap<KeyType, ValueType, SetAllocator, KeyFuncs>, KeyType, ValueType, SetAllocator, KeyFuncs, LockingPolicy>
{
public:
	using Super = TStripedMapBase<BucketCount, TMultiMap<KeyType, ValueType, SetAllocator, KeyFuncs>, KeyType, ValueType, SetAllocator, KeyFuncs, LockingPolicy>;

	/**
	 * Finds all values associated with the specified key.
	 *
	 * @param Key The key to find associated values for.
	 * @param OutValues Upon return, contains the values associated with the key.
	 * @param bMaintainOrder true if the Values array should be in the same order as the map's pairs.
	 */
	template<typename Allocator>
	void MultiFind(typename Super::KeyInitType InKey, TArray<ValueType, Allocator>& OutValues, bool bMaintainOrder = false) const
	{
		Super::Read(InKey, [&InKey, &OutValues, bMaintainOrder](uint32 Hash, const typename Super::MapType& Map) { Map.MultiFind(InKey, OutValues, bMaintainOrder); });
	}
};
