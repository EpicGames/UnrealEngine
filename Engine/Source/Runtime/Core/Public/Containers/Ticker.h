// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/MpscQueue.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

#include <atomic>

/**
 * Ticker delegates return true to automatically reschedule at the same delay or false for one-shot.
 * You will not get more than one fire per "frame", which is just a FTSTicker::Tick call.
 * DeltaTime argument is the time since the last game frame, *not* since the last tick the
 * delegate received.
 */
DECLARE_DELEGATE_RetVal_OneParam(bool, FTickerDelegate, float);

/**
 * Thread-safe ticker class. Fires delegates after a delay.
 */
class FTSTicker
{
private:
	struct FElement;

public:
	using FDelegateHandle = TWeakPtr<FElement>;

	/** Singleton used for the ticker in Core / Launch. If you add a new ticker for a different subsystem, do not put that singleton here! **/
	[[nodiscard]] static CORE_API FTSTicker& GetCoreTicker();

	/**
	 * Add a new ticker with a given delay / interval
	 *
	 * Can be called concurrently. Resources used by the delegate must be released on the ticking thread.
	 *
	 * @param InDelegate Delegate to fire after the delay
	 * @param InDelay Delay until next fire; 0 means "next frame"
	 */
	CORE_API FDelegateHandle AddTicker(const FTickerDelegate & InDelegate, float InDelay = 0.0f);

	/**
	* Add a new ticker with a given delay / interval.
	* 
	* Can be called concurrently.
	*
	* @param InName Name of this ticker for profiling
	* @param InDelay Delay until next fire; 0 means "next frame"
	* @param Function Function to execute. Should return true to fire after another InDelay time
	*/
	CORE_API FDelegateHandle AddTicker(const TCHAR* InName, float InDelay, TUniqueFunction<bool(float)>&& InFunction);

	/**
	 * Removes a previously added ticker delegate.
	 *
	 * Can be called concurrently. If is called in the middle of the delegate execution, it blocks until the execution finishes, and thus guarantees
	 * that the delegate won't be executed after the call.
	 *
	 * @param Handle The handle of the ticker to remove.
	 */
	static CORE_API void RemoveTicker(FDelegateHandle Handle);

	/**
	 * Fire all tickers who have passed their delay and reschedule the ones that return true
	 * 
	 * Note: This reschedule has timer skew, meaning we always wait a full Delay period after
	 * each invocation even if we missed the last interval by a bit. For instance, setting a
	 * delay of 0.1 seconds will not necessarily fire the delegate 10 times per second, depending
	 * on the Tick() rate. What this DOES guarantee is that a delegate will never be called MORE
	 * FREQUENTLY than the Delay interval, regardless of how far we miss the scheduled interval.
	 * 
	 * Must not be called concurrently.
	 *
	 * @param DeltaTime	time that has passed since the last tick call
	 */
	CORE_API void Tick(float DeltaTime);

	/**
	 * Resets the instance to its default state. Must be called from the ticking thread.
	 */
	CORE_API void Reset();

private:
	/** Internal structure to store a ticker delegate and related data **/
	struct FElement
	{
		/** Time that this delegate must not fire before **/
		double FireTime;
		/** Delay that this delegate was scheduled with. Kept here so that if the delegate returns true, we will reschedule it. **/
		float DelayTime;
		/** Function to fire **/
		TUniqueFunction<bool(float)> Function;

		static constexpr uint64 DefaultState = 0;
		static constexpr uint64 RemovedState = 1;
		// The element can be in 4 states: "idle", "executing", "idle and removed" and "executing and removed"
		// Often right after `RemoveTicket()` call resources that are used by the delegate are destroyed. We must ensure that 
		// these resources are not accessed by the delegate after `RemoveTicket()` returns.
		// `State` is used to remove the element safely, if removal happens in the middle of the delegate execution, removal will not return
		// until the execution is finished.
		// `State` constists of two 32 bits parts: lower part is 1 if the delegate is removed, otherwise it's 0. the higher part contains thread id if the
		// delegate being executed, otherwise it's 0
		std::atomic<uint64> State{ DefaultState };

		/** Default ctor is only required to implement CurrentElement handling without making it a pointer. */
		[[nodiscard]] CORE_API FElement();
		/** This is the ctor that the code will generally use. */
		[[nodiscard]] CORE_API FElement(double InFireTime, float InDelayTime, const FTickerDelegate& InDelegate);
		/** This is the ctor that the code will generally use. */
		[[nodiscard]] CORE_API FElement(double InFireTime, float InDelayTime, TUniqueFunction<bool(float)>&& InFunction);

		/** Fire the delegate if it is fireable **/
		CORE_API bool Fire(float DeltaTime);
	};

	using FElementPtr = TSharedPtr<FElement>;

	// all added delegates are initially stored in a separate thread-safe queue and then in the next Tick are moved to the main not thread-safe
	// container
	TMpscQueue<FElementPtr> AddedElements;

	/** Current time of the ticker **/
	std::atomic<double> CurrentTime{ 0.0 };
	/** Future delegates to fire **/
	TArray<FElementPtr> Elements;
};

/**
 * Base class for thread-safe ticker objects
 */
class FTSTickerObjectBase
{
public:
	UE_NONCOPYABLE(FTSTickerObjectBase);

	/**
	 * Constructor
	 *
	 * @param InDelay Delay until next fire; 0 means "next frame"
	 * @param Ticker the ticker to register with. Defaults to FTSTicker::GetCoreTicker().
	*/
	[[nodiscard]] CORE_API FTSTickerObjectBase(float InDelay = 0.0f, FTSTicker& Ticker = FTSTicker::GetCoreTicker());

	/** Virtual destructor. */
	CORE_API virtual ~FTSTickerObjectBase();

	/**
	 * Pure virtual that must be overloaded by the inheriting class.
	 *
	 * @param DeltaTime	time passed since the last call.
	 * @return true if should continue ticking
	 */
	virtual bool Tick(float DeltaTime) = 0;

private:
	/** Delegate for callbacks to Tick */
	FTSTicker::FDelegateHandle TickHandle;
};

// Schedules execution of the given functor on the game thread at a particular moment of a frame when interference with other systems 
// is minimal.
template<typename FunctorType>
void ExecuteOnGameThread(const TCHAR* DebugName, FunctorType&& Functor)
{
	FTSTicker::GetCoreTicker().AddTicker(DebugName, 0.0f,
		[Functor = MoveTemp(Functor)] (float) mutable
		{
			Functor();
			return false;
		}
	);
}
