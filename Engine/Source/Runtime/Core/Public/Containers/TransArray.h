// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTypeTraits.h"
#include "Containers/Array.h"
#include "Misc/ITransaction.h"


extern CORE_API ITransaction* GUndo;

/*-----------------------------------------------------------------------------
	Transactional array.
-----------------------------------------------------------------------------*/

// NOTE: Right now, you can't use a custom allocation policy with transactional arrays. If
// you need to do it, you will have to fix up FTransaction::FObjectRecord to use the correct TArray<Allocator>.
template< typename T >
class TTransArray
	: public TArray<T>
{
public:

	typedef TArray<T> Super;

	// Constructors.
	[[nodiscard]] explicit TTransArray( UObject* InOwner )
	:	Owner( InOwner )
	{
		checkSlow(Owner);
	}

	[[nodiscard]] TTransArray( UObject* InOwner, const Super& Other )
	:	Super( Other )
	,	Owner( InOwner )
	{
		checkSlow(Owner);
	}

	[[nodiscard]] TTransArray(TTransArray&&) = default;
	[[nodiscard]] TTransArray(const TTransArray&) = default;
	TTransArray& operator=(TTransArray&&) = default;
	TTransArray& operator=(const TTransArray&) = default;

	// Add, Insert, Remove, Empty interface.
	int32 AddUninitialized( int32 Count=1 )
	{
		const int32 Index = Super::AddUninitialized( Count );
		if( GUndo )
		{
			GUndo->SaveArray( Owner, (FScriptArray*)this, Index, Count, 1, sizeof(T), alignof(T), DefaultConstructItem, SerializeItem, DestructItem );
		}
		return Index;
	}

	void InsertUninitialized( int32 Index, int32 Count=1 )
	{
		Super::InsertUninitialized( Index, Count );
		if( GUndo )
		{
			GUndo->SaveArray( Owner, (FScriptArray*)this, Index, Count, 1, sizeof(T), alignof(T), DefaultConstructItem, SerializeItem, DestructItem );
		}
	}

	void RemoveAt( int32 Index, int32 Count=1 )
	{
		if( GUndo )
		{
			GUndo->SaveArray( Owner, (FScriptArray*)this, Index, Count, -1, sizeof(T), alignof(T), DefaultConstructItem, SerializeItem, DestructItem );
		}
		Super::RemoveAt( Index, Count );
	}

	void Empty( int32 Slack=0 )
	{
		if( GUndo )
		{
			GUndo->SaveArray( Owner, (FScriptArray*)this, 0, this->ArrayNum, -1, sizeof(T), alignof(T), DefaultConstructItem, SerializeItem, DestructItem );
		}
		Super::Empty( Slack );
	}

	// Functions dependent on Add, Remove.
	void AssignButKeepOwner( const Super& Other )
	{
		(Super&)*this = Other;
	}

	// Functions dependent on Add, Remove.
	void AssignButKeepOwner( Super&& Other )
	{
		(Super&)*this = MoveTemp(Other);
	}

	int32 Add( const T& Item )
	{
		new(*this) T(Item);
		return this->Num() - 1;
	}

	int32 AddZeroed( int32 n=1 )
	{
		const int32 Index = AddUninitialized(n);
		FMemory::Memzero(this->GetData() + Index, n*sizeof(T));
		return Index;
	}

	int32 AddUnique( const T& Item )
	{
		for( int32 Index=0; Index<this->ArrayNum; Index++ )
		{
			if( (*this)[Index]==Item )
			{
				return Index;
			}
		}
		return Add( Item );
	}

	int32 Remove( const T& Item )
	{
		this->CheckAddress(&Item);

		const int32 OriginalNum=this->ArrayNum;
		for( int32 Index=0; Index<this->ArrayNum; Index++ )
		{
			if( (*this)[Index]==Item )
			{
				RemoveAt( Index-- );
			}
		}
		return OriginalNum - this->ArrayNum;
	}

	// TTransArray interface.
	[[nodiscard]] UObject* GetOwner() const
	{
		return Owner;
	}

	void SetOwner( UObject* NewOwner )
	{
		Owner = NewOwner;
	}

	void ModifyItem( int32 Index )
	{
		if( GUndo )
		{
			GUndo->SaveArray( Owner, (FScriptArray*)this, Index, 1, 0, sizeof(T), alignof(T), DefaultConstructItem, SerializeItem, DestructItem );
		}
	}

	void ModifyAllItems()
	{
		if( GUndo )
		{
			GUndo->SaveArray( Owner, (FScriptArray*)this, 0, this->Num(), 0, sizeof(T), alignof(T), DefaultConstructItem, SerializeItem, DestructItem );
		}
	}

	friend FArchive& operator<<( FArchive& Ar, TTransArray& A )
	{
		Ar << A.Owner;
		Ar << (Super&)A;
		return Ar;
	}

protected:

	static void DefaultConstructItem( void* TPtr )
	{
		::new (TPtr) T;
	}
	static void SerializeItem( FArchive& Ar, void* TPtr )
	{
		Ar << *(T*)TPtr;
	}
	static void DestructItem( void* TPtr )
	{
		((T*)TPtr)->~T();
	}
	UObject* Owner;
};



//
// Transactional array operator news.
//
template <typename T> void* operator new( size_t Size, TTransArray<T>& Array )
{
	check(Size == sizeof(T));
	const int32 Index = Array.AddUninitialized();
	return &Array[Index];
}


template <typename T> void* operator new( size_t Size, TTransArray<T>& Array, int32 Index )
{
	check(Size == sizeof(T));
	Array.Insert(Index);
	return &Array[Index];
}
