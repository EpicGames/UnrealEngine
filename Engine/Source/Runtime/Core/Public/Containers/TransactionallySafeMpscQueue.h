// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Async/TransactionallySafeMutex.h"
#include "Misc/Optional.h"
#include "Misc/ScopeLock.h"
#include "Templates/MemoryOps.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/UnrealTemplate.h"

/**
 * Fast, transactionally-safe multi-producer/single-consumer unbounded concurrent queue. 
 * Doesn't free memory until destruction but recycles consumed items.
 * 
 * The transactionally-safe queue uses a mutex to enforce thread safety instead of atomic 
 * operations. The difference in performance should be negligible unless you are CPU-bound
 * on constantly enqueueing and dequeueing objects as fast as possible.
 * 
 * It is not safe to spin-wait on Dequeue from within an AutoRTFM transaction!
 * The other thread's Enqueue will be blocked on the mutex, so you will deadlock inside the 
 * spin-wait.
 */
template<typename T, typename AllocatorType = FMemory>
class TTransactionallySafeMpscQueue final
{
public:
	using ElementType = T;

	UE_NONCOPYABLE(TTransactionallySafeMpscQueue);

	[[nodiscard]] TTransactionallySafeMpscQueue()
	{
		FNode* Node = ::new (AllocatorType::Malloc(sizeof(FNode), alignof(FNode))) FNode;
		Head = First = Tail = Node;
	}

	~TTransactionallySafeMpscQueue()
	{
		// Nobody should have a reference to this class anymore, but we still take the mutex to
		// guarantee that the queue isn't modified by another thread while a transaction is underway.
		UE::TScopeLock Lock(Mutex);
		// Free all recycled/sentinel nodes (First up to and including Tail).
		FNode* Node = First;
		while (Node != Tail)
		{
			FNode* Next = Node->Next;
			AllocatorType::Free(Node);
			Node = Next;
		}

		// Free the sentinel (Tail) and advance to occupied nodes.
		Node = Tail->Next;
		AllocatorType::Free(Tail);

		// Destroy and free all occupied nodes.
		while (Node != nullptr)
		{
			FNode* Next = Node->Next;
			DestructItem((ElementType*)&Node->Value);
			AllocatorType::Free(Node);
			Node = Next;
		}
	}

	template <typename... ArgTypes>
	void Enqueue(ArgTypes&&... Args)
	{
		FNode* Node = AllocNode();
		::new ((void*)&Node->Value) ElementType(Forward<ArgTypes>(Args)...);

		UE::TScopeLock Lock(Mutex);
		Head->Next = Node;
		Head = Node;
	}

	// Returns NullOpt if the queue is empty.
	// Spin-waiting on Dequeue from within an AutoRTFM transaction is likely to deadlock,
	// as the matching Enqueue will be waiting on Mutex.
	TOptional<ElementType> Dequeue()
	{
		Mutex.Lock();
		FNode* LocalTail = Tail;
		FNode* LocalTailNext = LocalTail->Next;
		Mutex.Unlock();

		if (LocalTailNext == nullptr)
		{
			return NullOpt;
		}

		ElementType* TailNextValue = (ElementType*)&LocalTailNext->Value;
		TOptional<ElementType> Value{ MoveTemp(*TailNextValue) };
		DestructItem(TailNextValue);

		Mutex.Lock();
		Tail = LocalTailNext;
		Mutex.Unlock();

		return Value;
	}

	bool Dequeue(ElementType& OutElem)
	{
		TOptional<ElementType> LocalElement = Dequeue();
		if (LocalElement.IsSet())
		{
			OutElem = MoveTempIfPossible(LocalElement.GetValue());
			return true;
		}
		
		return false;
	}

	[[nodiscard]] bool IsEmpty() const
	{
		UE::TScopeLock Lock(Mutex);
		FNode* LocalTail = Tail;
		FNode* LocalTailNext = LocalTail->Next;

		return LocalTailNext == nullptr;
	}

	// As there can be only one consumer, a consumer can safely "peek" the tail of the queue.
	// Returns a pointer to the tail if the queue is not empty, nullptr otherwise.
	// There's no overload with TOptional as it doesn't support references.
	[[nodiscard]] ElementType* Peek() const
	{
		UE::TScopeLock Lock(Mutex);
		FNode* LocalTail = Tail;
		FNode* LocalTailNext = LocalTail->Next;

		return LocalTailNext
			? (ElementType*)&LocalTailNext->Value
			: nullptr;
	}

private:
	struct FNode
	{
		FNode* Next = nullptr;
		TTypeCompatibleBytes<ElementType> Value;
	};

	FNode* AllocNode()
	{
		// Attempt to recycle a consumed node from the free list (First..Tail range).
		{
			UE::TScopeLock Lock(Mutex);
			if (First != Tail)
			{
				FNode* Node = First;
				First = First->Next;
				Node->Next = nullptr;
				return Node;
			}
		}

		// Free list is empty; allocate a new node.
		return ::new (AllocatorType::Malloc(sizeof(FNode), alignof(FNode))) FNode();
	}

	// This mutex guards all accesses to this structure or to FNode::Next.
	mutable UE::FTransactionallySafeMutex Mutex;

	FNode* Tail; // tail of the queue (consumer end)
	FNode* Head; // head of the queue (producer end)
	FNode* First; // head of the recycled node free list
};
