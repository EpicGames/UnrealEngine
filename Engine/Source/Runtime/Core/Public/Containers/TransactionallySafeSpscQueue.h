// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TransactionallySafeMpscQueue.h"

/**
 * Fast, transactionally-safe single-producer/single-consumer unbounded concurrent queue. 
 * Doesn't free memory until destruction but recycles consumed items. Based on TSpscQueue.
 * 
 * This is just an alias to `TTransactionallySafeMpscQueue`: because we use locks to enforce
 * atomicity the queues are implementation wise the same.
 * 
 * It is not safe to spin-wait on Dequeue from within an AutoRTFM transaction!
 * The other thread's Enqueue will be blocked on the mutex, so you will deadlock inside the 
 * spin-wait. This class works best with the game thread as the producer, and a separate 
 * helper thread as the consumer.
 */
template<typename T, typename AllocatorType = FMemory> using TTransactionallySafeSpscQueue = TTransactionallySafeMpscQueue<T, AllocatorType>;
