// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PlatformMisc.h"

/**
 * Template for triple buffers.
 *
 * This template implements a lock-free triple buffer that can be used to exchange data
 * between two threads that are producing and consuming at different rates. Instead of
 * atomically exchanging pointers to the buffers, we atomically update a Flags register
 * that holds the indices into a 3-element buffer array.
 *
 * The three buffers are named as follows:
 * - Read buffer: This is where Read() will read the latest value from
 * - Write buffer: This is where Write() will write a new value to
 * - Temp buffer: This is the second back-buffer currently not used for reading or writing
 *
 * Please note that reading and writing to the buffer does not automatically swap the
 * back-buffers. Instead, two separate methods, SwapReadBuffers() and SwapWriteBuffers()
 * are provided. For convenience, we also provide SwapAndRead() and WriteAndSwap() to
 * update and swap the buffers using a single method call.
 *
 * A dirty flag indicates whether a new value has been written and swapped into the second
 * back-buffer and is available for reading. It can be checked using the IsDirtyMethod().
 * As an optimization, SwapReadBuffers() and SwapAndRead() will not perform a back-buffer
 * swap if no new data is available.
 *
 * This class is thread-safe in single-producer, single-consumer scenarios.
 *
 * Based on ideas and C code in "Triple Buffering as a Concurrency Mechanism" (Reddit.com)
 *
 * @param BufferType The type of buffered items.
 */
template<typename BufferType>
class TTripleBuffer
{
	/** Enumerates human readable bit values for accessing the Flags field. */
	enum EBufferFlag
	{
		/** Indicates whether a new buffer is available for reading. */
		Dirty = 0x40,

		/** Initial flags value (0dttwwrr; dirty = false, temp index = 0, write index = 1, read index = 2) */
		Initial = 0x06,

		/** Bit mask for accessing the read buffer index (bit 0-1). */
		ReaderMask = 0x03,

		/** Bit mask for the index of the unused/clean/empty buffer (bit 4-5). */
		TempMask = 0x30,

		/** Bit shift for accessing the temp buffer index. */
		TempShift = 4,

		/** Bit mask for accessing the write buffer index (bit 2-3). */
		WriterMask = 0x0c,

		/** Bit shift for accessing the write buffer index. */
		WriterShift = 2,
	};

public:

	/** Default constructor. */
	[[nodiscard]] TTripleBuffer()
	{
		Initialize();
		Buffers[0] = Buffers[1] = Buffers[2] = BufferType();
	}

	/** Default constructor (no initialization). */
	[[nodiscard]] explicit TTripleBuffer(ENoInit)
	{
		Initialize();
	}

	/**
	 * Create and initialize a new instance with a given buffer value.
	 *
	 * @param InValue The initial value of all three buffers.
	 */
	[[nodiscard]] explicit TTripleBuffer(const BufferType& InValue)
	{
		Initialize();
		Buffers[0] = Buffers[1] = Buffers[2] = InValue;
	}

	/**
	 * Create and initialize a new instance using provided buffers.
	 *
	 * The elements of the provided items array are expected to have
	 * the following initial contents:
	 *     0 = Temp
	 *     1 = Write
	 *     2 = Read
	 *
	 * @param InBuffers The buffer memory to use.
	 */
	[[nodiscard]] TTripleBuffer(BufferType (&InBuffers)[3])
	{
		Buffers = &InBuffers[0];
		Flags = EBufferFlag::Initial | EBufferFlag::Dirty;
		OwnsMemory = false;
	}

	/** Destructor. */
	~TTripleBuffer()
	{
		if (OwnsMemory)
		{
			delete[] Buffers;
		}
	}

public:

	/**
	 * Check whether a new value is available for reading.
	 *
	 * @return true if a new value is available, false otherwise.
	 */
	[[nodiscard]] bool IsDirty() const
	{
		return ((Flags & EBufferFlag::Dirty) != 0);
	}

	/**
	 * Read a value from the current read buffer.
	 *
	 * @return Reference to the read buffer's value.
	 * @see SwapRead, Write
	 */
	[[nodiscard]] BufferType& Read()
	{
		return Buffers[Flags & EBufferFlag::ReaderMask];
	}

	/**
	 * Swap the latest read buffer, if available.
	 *
	 * Will not perform a back-buffer swap if no new data is available (dirty flag = false).
	 *
	 * @see SwapWrite, Read
	 */
	void SwapReadBuffers()
	{
		if (!IsDirty())
		{
			return;
		}

		int32 CurrentFlags = 0;

		do
		{
			CurrentFlags = Flags;
		}
		while (FPlatformAtomics::InterlockedCompareExchange(&Flags, SwapReadWithTempFlags(CurrentFlags), CurrentFlags) != CurrentFlags);
	}

public:

	/**
	 * Get the current write buffer.
	 *
	 * @return Reference to write buffer.
	 */
	[[nodiscard]] BufferType& GetWriteBuffer()
	{
		return Buffers[(Flags & EBufferFlag::WriterMask) >> EBufferFlag::WriterShift];
	}

	/**
	 * Swap a new write buffer (makes current write buffer available for reading).
	 *
	 * @see SwapRead, Write
	 */
	void SwapWriteBuffers()
	{
		int32 CurrentFlags = 0;

		do
		{
			CurrentFlags = Flags;
		}
		while (FPlatformAtomics::InterlockedCompareExchange(&Flags, SwapWriteWithTempFlags(CurrentFlags), CurrentFlags) != CurrentFlags);
	}

	/**
	 * Write a value to the current write buffer.
	 *
	 * @param Value The value to write.
	 * @see SwapWrite, Read
	 */
	void Write(const BufferType Value)
	{
		Buffers[(Flags & EBufferFlag::WriterMask) >> EBufferFlag::WriterShift] = Value;
	}

public:

	/** Reset the buffer. */
	void Reset()
	{
		Flags = EBufferFlag::Initial;
		FPlatformMisc::MemoryBarrier();
	}

	/**
	 * Convenience method for fetching and reading the latest buffer.
	 *
	 * @return Reference to the buffer.
	 * @see SwapRead, Read, WriteAndSwap
	 */
	[[nodiscard]] const BufferType& SwapAndRead()
	{
		SwapReadBuffers();
		return Read();
	}

	/**
	 * Convenience method for writing the latest buffer and fetching a new one.
	 *
	 * @param Value The value to write into the buffer.
	 * @see SwapAndRead, SwapWrite, Write
	 */
	void WriteAndSwap(const BufferType Value)
	{
		Write(Value);
		SwapWriteBuffers();
	}

protected:

	/** Initialize the triple buffer. */
	void Initialize()
	{
		Buffers = new BufferType[3];
		Flags = EBufferFlag::Initial;
		OwnsMemory = true;
	}

private:

	/** Swaps the read and temp buffer indices in the Flags field. */
	[[nodiscard]] static UE_FORCEINLINE_HINT int32 SwapReadWithTempFlags(int32 Flags)
	{
		return ((Flags & EBufferFlag::ReaderMask) << 4) | ((Flags & EBufferFlag::TempMask) >> 4) | (Flags & EBufferFlag::WriterMask);
	}

	/** Swaps the write and temp buffer indices in the Flags field, and sets the dirty bit. */
	[[nodiscard]] static UE_FORCEINLINE_HINT int32 SwapWriteWithTempFlags(int32 Flags)
	{
		return ((Flags & EBufferFlag::TempMask) >> 2) | ((Flags & EBufferFlag::WriterMask) << 2) | (Flags & EBufferFlag::ReaderMask) | EBufferFlag::Dirty;
	}

private:

	/** Hidden copy constructor (triple buffers cannot be copied). */
	TTripleBuffer(const TTripleBuffer&);

	/** Hidden copy assignment (triple buffers cannot be copied). */
	TTripleBuffer& operator=(const TTripleBuffer&);

private:

	/** The three buffers. */
	BufferType* Buffers;

	/** Buffer access flags. */
	MS_ALIGN(16) int32 volatile Flags GCC_ALIGN(16);

	/** Whether this instance owns the buffer memory. */
	bool OwnsMemory;
};
