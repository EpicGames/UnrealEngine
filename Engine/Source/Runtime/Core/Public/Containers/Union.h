// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "Trace/Detail/Channel.h"


CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogUnion, Log, All);


/** Used to disambiguate methods that are overloaded for all possible subtypes of a TUnion where the subtypes may not be distinct. */
template<uint32>
struct TDisambiguater
{
	TDisambiguater() {}
};


class FNull
{
public:

	friend uint32 GetTypeHash(FNull)
	{
		return 0;
	}

	bool operator==(const FNull&) const
	{
		return true;
	}
	
	bool operator!=(const FNull&) const
	{
		return false;
	}

	friend FArchive& operator<<(FArchive& Ar, FNull& Value)
	{
		// nothing
		return Ar;
	}
};


/**
 * Represents a type which is the union of several other types; i.e. it can have a value whose type is of any the union's subtypes.
 * This differs from C union types by being type-safe, and supporting non-trivial data types as subtypes.
 * Since a value for the union must be of a single subtype, the union stores potential values of different subtypes in overlapped memory, and keeps track of which one is currently valid.
 * 
 * TUnion should be considered deprecated and TVariant should be used instead for new code.
 */
template<typename TypeA,typename TypeB = FNull,typename TypeC = FNull,typename TypeD = FNull,typename TypeE = FNull,typename TypeF = FNull>
class TUnion
{
public:

	/** Default constructor. */
	TUnion()
	{ }

	/** Initialization constructor. */
	explicit TUnion(typename TCallTraits<TypeA>::ParamType InValue, TDisambiguater<0> Disambiguater = TDisambiguater<0>())
	{
		SetSubtype<TypeA>(InValue);
	}
	
	/** Initialization constructor. */
	explicit TUnion(typename TCallTraits<TypeB>::ParamType InValue, TDisambiguater<1> Disambiguater = TDisambiguater<1>())
	{
		SetSubtype<TypeB>(InValue);
	}
	
	/** Initialization constructor. */
	explicit TUnion(typename TCallTraits<TypeC>::ParamType InValue, TDisambiguater<2> Disambiguater = TDisambiguater<2>())
	{
		SetSubtype<TypeC>(InValue);
	}
	
	/** Initialization constructor. */
	explicit TUnion(typename TCallTraits<TypeD>::ParamType InValue, TDisambiguater<3> Disambiguater = TDisambiguater<3>())
	{
		SetSubtype<TypeD>(InValue);
	}
	
	/** Initialization constructor. */
	explicit TUnion(typename TCallTraits<TypeE>::ParamType InValue, TDisambiguater<4> Disambiguater = TDisambiguater<4>())
	{
		SetSubtype<TypeE>(InValue);
	}
	
	/** Initialization constructor. */
	explicit TUnion(typename TCallTraits<TypeF>::ParamType InValue, TDisambiguater<5> Disambiguater = TDisambiguater<5>())
	{
		SetSubtype<TypeF>(InValue);
	}

	/** Copy constructor. */
	TUnion(const TUnion& Other)
	{
		*this = Other;
	}

	/** Destructor. */
	~TUnion()
	{
		// Destruct any subtype value the union may have.
		Reset();
	}

	/** @return True if the union's value is of the given subtype. */
	template<typename Subtype>
	bool HasSubtype() const
	{
		// Determine the subtype's index and reference.
		uint8 SubtypeIndex;
		const Subtype* SubtypeValuePointer;
		GetSubtypeIndexAndReference<Subtype,const Subtype*>(*this,SubtypeIndex,SubtypeValuePointer);

		return CurrentSubtypeIndex == SubtypeIndex;
	}

	/** If the union's current value is of the given subtype, sets the union's value to a NULL value. */
	template<typename Subtype>
	void ResetSubtype()
	{
		// Determine the subtype's index and reference.
		uint8 SubtypeIndex;
		Subtype* SubtypeValuePointer;
		GetSubtypeIndexAndReference<Subtype,Subtype*>(*this,SubtypeIndex,SubtypeValuePointer);

		// Only reset the value if it is of the specified subtype.
		if(CurrentSubtypeIndex == SubtypeIndex)
		{
			CurrentSubtypeIndex = -1;

			// Destruct the subtype.
			SubtypeValuePointer->~Subtype();
		}
	}

	/** @return A reference to the union's value of the given subtype.  May only be called if the union's HasSubtype()==true for the given subtype. */
	template<typename Subtype>
	const Subtype& GetSubtype() const
	{
		// Determine the subtype's index and reference.
		uint8 SubtypeIndex;
		const Subtype* SubtypeValuePointer;
		GetSubtypeIndexAndReference<Subtype,const Subtype*>(*this,SubtypeIndex,SubtypeValuePointer);

		// Validate that the union has a value of the requested subtype.
		check(CurrentSubtypeIndex == SubtypeIndex);

		return *SubtypeValuePointer;
	}

	/** @return A reference to the union's value of the given subtype.  May only be called if the union's HasSubtype()==true for the given subtype. */
	template<typename Subtype>
	Subtype& GetSubtype()
	{
		// Determine the subtype's index and reference.
		uint8 SubtypeIndex;
		Subtype* SubtypeValuePointer;
		GetSubtypeIndexAndReference<Subtype,Subtype*>(*this,SubtypeIndex,SubtypeValuePointer);

		// Validate that the union has a value of the requested subtype.
		check(CurrentSubtypeIndex == SubtypeIndex);

		return *SubtypeValuePointer;
	}

	/** Replaces the value of the union with a value of the given subtype. */
	template<typename Subtype>
	Subtype* SetSubtype()
	{
		uint8 SubtypeIndex;
		Subtype* SubtypeValuePointer;
		GetSubtypeIndexAndReference<Subtype, Subtype*>(*this, SubtypeIndex, SubtypeValuePointer);

		Reset();

		::new((void*)SubtypeValuePointer) Subtype();

		CurrentSubtypeIndex = SubtypeIndex;
		return SubtypeValuePointer;
	}

	/** Replaces the value of the union with a value of the given subtype. */
	template<typename Subtype>
	Subtype* SetSubtype(typename TCallTraits<Subtype>::ParamType NewValue)
	{
		uint8 SubtypeIndex;
		Subtype* SubtypeValuePointer;
		GetSubtypeIndexAndReference<Subtype,Subtype*>(*this,SubtypeIndex,SubtypeValuePointer);

		Reset();

		::new((void*)SubtypeValuePointer) Subtype(NewValue);

		CurrentSubtypeIndex = SubtypeIndex;
		return SubtypeValuePointer;
	}

	/** @return The index corresponding to the type currently stored in this union; useful for writing switches and indexing into tables. */
	uint8 GetCurrentSubtypeIndex() const
	{
		return CurrentSubtypeIndex;
	}

	/** Sets the union's value to NULL. */
	void Reset()
	{
		switch(CurrentSubtypeIndex)
		{
		case 0: ResetSubtype<TypeA>(); break;
		case 1: ResetSubtype<TypeB>(); break;
		case 2: ResetSubtype<TypeC>(); break;
		case 3: ResetSubtype<TypeD>(); break;
		case 4: ResetSubtype<TypeE>(); break;
		case 5: ResetSubtype<TypeF>(); break;
		};
	}

	/** Hash function. */
	friend uint32 GetTypeHash(const TUnion& Union)
	{
		uint32 Result = GetTypeHash(Union.CurrentSubtypeIndex);

		switch(Union.CurrentSubtypeIndex)
		{
		case 0: Result ^= GetTypeHash(Union.GetSubtype<TypeA>()); break;
		case 1: Result ^= GetTypeHash(Union.GetSubtype<TypeB>()); break;
		case 2: Result ^= GetTypeHash(Union.GetSubtype<TypeC>()); break;
		case 3: Result ^= GetTypeHash(Union.GetSubtype<TypeD>()); break;
		case 4: Result ^= GetTypeHash(Union.GetSubtype<TypeE>()); break;
		case 5: Result ^= GetTypeHash(Union.GetSubtype<TypeF>()); break;
		default: FatalErrorUndefinedSubtype(); break;
		};

		return Result;
	}

	TUnion& operator=(const TUnion& Other)
	{
		// Copy the value of the appropriate subtype from the other union
		switch (Other.CurrentSubtypeIndex)
		{
		case (uint8)-1: break;
		case 0: SetSubtype<TypeA>(Other.GetSubtype<TypeA>()); break;
		case 1: SetSubtype<TypeB>(Other.GetSubtype<TypeB>()); break;
		case 2: SetSubtype<TypeC>(Other.GetSubtype<TypeC>()); break;
		case 3: SetSubtype<TypeD>(Other.GetSubtype<TypeD>()); break;
		case 4: SetSubtype<TypeE>(Other.GetSubtype<TypeE>()); break;
		case 5: SetSubtype<TypeF>(Other.GetSubtype<TypeF>()); break;
		default: FatalErrorUndefinedSubtype(); break;
		};

		return *this;
	}

	/** Equality comparison. */
	bool operator==(const TUnion& Other) const
	{
		if(CurrentSubtypeIndex == Other.CurrentSubtypeIndex)
		{
			switch(CurrentSubtypeIndex)
			{
			case 0: return GetSubtype<TypeA>() == Other.GetSubtype<TypeA>(); break;
			case 1: return GetSubtype<TypeB>() == Other.GetSubtype<TypeB>(); break;
			case 2: return GetSubtype<TypeC>() == Other.GetSubtype<TypeC>(); break;
			case 3: return GetSubtype<TypeD>() == Other.GetSubtype<TypeD>(); break;
			case 4: return GetSubtype<TypeE>() == Other.GetSubtype<TypeE>(); break;
			case 5: return GetSubtype<TypeF>() == Other.GetSubtype<TypeF>(); break;
			default: FatalErrorUndefinedSubtype(); break;
			};
		}

		return false;
	}

	void Serialize(FArchive& Ar)
	{
		if(Ar.IsLoading())
		{
			Reset();

			Ar << CurrentSubtypeIndex;

			switch(CurrentSubtypeIndex)
			{
			case 0: Ar << InitSubtype<TypeA>(); break;
			case 1: Ar << InitSubtype<TypeB>(); break;
			case 2: Ar << InitSubtype<TypeC>(); break;
			case 3: Ar << InitSubtype<TypeD>(); break;
			case 4: Ar << InitSubtype<TypeE>(); break;
			case 5: Ar << InitSubtype<TypeF>(); break;
			default: FatalErrorUndefinedSubtype(); break;
			};
		}
		else
		{
			Ar << CurrentSubtypeIndex;

			switch(CurrentSubtypeIndex)
			{
			case 0: Ar << GetSubtype<TypeA>(); break;
			case 1: Ar << GetSubtype<TypeB>(); break;
			case 2: Ar << GetSubtype<TypeC>(); break;
			case 3: Ar << GetSubtype<TypeD>(); break;
			case 4: Ar << GetSubtype<TypeE>(); break;
			case 5: Ar << GetSubtype<TypeF>(); break;
			default: FatalErrorUndefinedSubtype(); break;
			};
		}
	}

private:

	/** The potential values for each subtype of the union. */
	union
	{
		TTypeCompatibleBytes<TypeA> A;
		TTypeCompatibleBytes<TypeB> B;
		TTypeCompatibleBytes<TypeC> C;
		TTypeCompatibleBytes<TypeD> D;
		TTypeCompatibleBytes<TypeE> E;
		TTypeCompatibleBytes<TypeF> F;
	} Values;

	/** The index of the subtype that the union's current value is of. */
	uint8 CurrentSubtypeIndex = (uint8)-1;

	/** Sets the union's value to a default value of the given subtype. */
	template<typename Subtype>
	Subtype& InitSubtype()
	{
		Subtype* NewSubtype = &GetSubtype<Subtype>();
		return *::new((void*)NewSubtype) Subtype;
	}

	/** Determines the index and reference to the potential value for the given union subtype. */
	template<typename Subtype,typename PointerType>
	static void GetSubtypeIndexAndReference(
		const TUnion& Union,
		uint8& OutIndex,
		PointerType& OutValuePointer
		)
	{
		if constexpr (std::is_same_v<TypeA,Subtype>)
		{
			OutIndex = 0;
			OutValuePointer = (PointerType)&Union.Values.A;
		}
		else if constexpr (std::is_same_v<TypeB,Subtype>)
		{
			OutIndex = 1;
			OutValuePointer = (PointerType)&Union.Values.B;
		}
		else if constexpr (std::is_same_v<TypeC,Subtype>)
		{
			OutIndex = 2;
			OutValuePointer = (PointerType)&Union.Values.C;
		}
		else if constexpr (std::is_same_v<TypeD,Subtype>)
		{
			OutIndex = 3;
			OutValuePointer = (PointerType)&Union.Values.D;
		}
		else if constexpr (std::is_same_v<TypeE,Subtype>)
		{
			OutIndex = 4;
			OutValuePointer = (PointerType)&Union.Values.E;
		}
		else if constexpr (std::is_same_v<TypeF,Subtype>)
		{
			OutIndex = 5;
			OutValuePointer = (PointerType)&Union.Values.F;
		}
		else
		{
			static_assert(sizeof(TypeA) == 0, "Type is not subtype of union.");
			OutIndex = (uint8)-1;
			OutValuePointer = NULL;
		}
	}

	static void FatalErrorUndefinedSubtype()
	{
		UE_LOGF(LogUnion, Fatal, "Unrecognized TUnion subtype");
	}
};

template<typename TypeA,typename TypeB,typename TypeC,typename TypeD,typename TypeE,typename TypeF>
FArchive& operator<<(FArchive& Ar, TUnion<TypeA, TypeB, TypeC, TypeD, TypeE, TypeF>& Union)
{
	Union.Serialize(Ar);
	return Ar;
}
