// Copyright Epic Games, Inc. All Rights Reserved.

// This needed to be UnrealString.h to avoid conflicting with
// the Windows platform SDK string.h

#pragma once

#include "Concepts/DecaysTo.h"

// Include UnrealString.h.inl's includes before defining the macros, in case the macros 'poison' other headers or there are re-entrant includes.
#include "Containers/UnrealStringIncludes.h.inl" // IWYU pragma: export

#define UE_STRING_CLASS                        FString
#define UE_STRING_CHARTYPE                     TCHAR
#define UE_STRING_CHARTYPE_IS_TCHAR            1
#define UE_STRING_PRINTF_FMT_CHARTYPE          TCHAR
#define UE_STRING_DEPRECATED(Version, Message)
	#include "Containers/UnrealString.h.inl" // IWYU pragma: export
#undef UE_STRING_DEPRECATED
#undef UE_STRING_PRINTF_FMT_CHARTYPE
#undef UE_STRING_CHARTYPE_IS_TCHAR
#undef UE_STRING_CHARTYPE
#undef UE_STRING_CLASS

/**
 * Convert an array of bytes to a string
 * @param In byte array values to convert
 * @param Count number of bytes to convert
 * @return Valid string representing bytes.
 */
[[nodiscard]] inline FString BytesToString(const uint8* In, int32 Count)
{
	FString Result;
	Result.Empty(Count);

	while (Count)
	{
		// Put the byte into an int16 and add 1 to it, this keeps anything from being put into the string as a null terminator
		int16 Value = *In;
		Value += 1;

		Result += FString::ElementType(Value);

		++In;
		Count--;
	}
	return Result;
}

/** 
 * Convert FString of bytes into the byte array.
 * @param String		The FString of byte values
 * @param OutBytes		Ptr to memory must be preallocated large enough
 * @param MaxBufferSize	Max buffer size of the OutBytes array, to prevent overflow
 * @return	The number of bytes copied
 */
inline int32 StringToBytes( const FString& String, uint8* OutBytes, int32 MaxBufferSize )
{
	int32 NumBytes = 0;
	const FString::ElementType* CharPos = *String;

	while( *CharPos && NumBytes < MaxBufferSize)
	{
		OutBytes[ NumBytes ] = (int8)(*CharPos - 1);
		CharPos++;
		++NumBytes;
	}
	return NumBytes;
}

/** Returns uppercase Char value of Nibble */
[[nodiscard]] inline TCHAR NibbleToTChar(uint8 Num)
{
	if (Num > 9)
	{
		return (TCHAR)('A' + (Num - 10));
	}
	return (TCHAR)('0' + Num);
}

/** Returns lowercase Char value of Nibble */
[[nodiscard]] inline TCHAR NibbleToTCharLower(uint8 Num)
{
	if (Num > 9)
	{
		return (TCHAR)('a' + (Num - 10));
	}
	return (TCHAR)('0' + Num);
}

/** 
 * Convert a byte to hex
 * @param In byte value to convert
 * @param Result out hex value output
 */
inline void ByteToHex(uint8 In, FString& Result)
{
	Result += NibbleToTChar(In >> 4);
	Result += NibbleToTChar(In & 15);
}

/** Convert bytes to uppercase hex string */
[[nodiscard]] inline FString BytesToHex(const uint8* Bytes, int32 NumBytes)
{
	FString Out;
	BytesToHex(Bytes, NumBytes, Out);
	return Out;
}

/** Convert bytes to lowercase hex string */
[[nodiscard]] inline FString BytesToHexLower(const uint8* Bytes, int32 NumBytes)
{
	FString Out;
	BytesToHexLower(Bytes, NumBytes, Out);
	return Out;
}

/**
 * Checks if the TChar is a valid hex character
 * @param Char		The character
 * @return	True if in 0-9 and A-F ranges
 */
[[nodiscard]] inline const bool CheckTCharIsHex(const TCHAR Char)
{
	return (Char >= TEXT('0') && Char <= TEXT('9')) || (Char >= TEXT('A') && Char <= TEXT('F')) || (Char >= TEXT('a') && Char <= TEXT('f'));
}

/**
 * Convert a TChar to equivalent hex value as a uint8
 * @param Hex		The character
 * @return	The uint8 value of a hex character
 */
[[nodiscard]] inline const uint8 TCharToNibble(const TCHAR Hex)
{
	if (Hex >= '0' && Hex <= '9')
	{
		return uint8(Hex - '0');
	}
	if (Hex >= 'A' && Hex <= 'F')
	{
		return uint8(Hex - 'A' + 10);
	}
	if (Hex >= 'a' && Hex <= 'f')
	{
		return uint8(Hex - 'a' + 10);
	}
	checkf(false, TEXT("'%c' (0x%02X) is not a valid hexadecimal digit"), Hex, Hex);
	return 0;
}

/** Convert numeric types to a string. Excludes bool to avoid hijacking through implicit conversion. */
template <
	typename StringType = FString,
	typename T
	UE_REQUIRES(std::is_arithmetic_v<T>)
>
[[nodiscard]] StringType LexToString(const T& Value) requires (!UE::CDecaysTo<T, bool>)
{
	// std::remove_cv_t to remove potential volatile decorations. Removing const is pointless, but harmless because it's specified in the param declaration.
	return StringType::Printf(TFormatSpecifier<std::remove_cv_t<T>>::template GetFormatSpecifier<typename StringType::FmtCharType>(), Value);
}

template <
	typename StringType = FString,
	typename CharType
	UE_REQUIRES(TIsCharType_V<CharType>)
>
[[nodiscard]] StringType LexToString(const CharType* Ptr)
{
	return StringType(Ptr);
}

/** Convert bool to a string. Do not allow implicit conversion to avoid highjacking other types. */
template <typename StringType = FString, typename BoolT>
[[nodiscard]] inline StringType LexToString(BoolT Value) requires UE::CDecaysTo<BoolT, bool>
{
	using ElementType = typename StringType::ElementType;
	return Value ? CHARTEXT(ElementType, "true") : CHARTEXT(ElementType, "false");
}

/** Helper template to convert to sanitized strings */
template <typename StringType = FString, typename T>
[[nodiscard]] StringType LexToSanitizedString(const T& Value)
{
	return LexToString<StringType>(Value);
}

/** Overloaded for floats */
template <typename StringType = FString>
[[nodiscard]] inline StringType LexToSanitizedString(float Value)
{
	return StringType::SanitizeFloat(Value);
}

/** Overloaded for doubles */
template <typename StringType = FString>
[[nodiscard]] inline StringType LexToSanitizedString(double Value)
{
	return StringType::SanitizeFloat(Value);
}

/** Shorthand legacy use for Lex functions */
template<typename T>
struct TTypeToString
{
	template <typename StringType = FString>
	[[nodiscard]] static StringType ToString(const T& Value)
	{
		return LexToString<StringType>(Value);
	}

	template <typename StringType = FString>
	[[nodiscard]] static StringType ToSanitizedString(const T& Value)
	{
		return LexToSanitizedString<StringType>(Value);
	}
};


template<typename T>
struct TTypeFromString
{
	template <typename CharType>
	static void FromString(T& Value, const CharType* Buffer)
	{
		return LexFromString(Value, Buffer);
	}
};

namespace StringConv
{
	/** Inline combine any UTF-16 surrogate pairs in the given string */
	UE_DEPRECATED(5.8, "Remove calls to this function because it does nothing.")
	inline void InlineCombineSurrogates(FString& Str)
	{
	}
}

struct FTextRange
{
	FTextRange()
		: BeginIndex(INDEX_NONE)
		, EndIndex(INDEX_NONE)
	{

	}

	FTextRange(int32 InBeginIndex, int32 InEndIndex)
		: BeginIndex(InBeginIndex)
		, EndIndex(InEndIndex)
	{

	}

	inline bool UEOpEquals(const FTextRange& Other) const
	{
		return BeginIndex == Other.BeginIndex
			&& EndIndex == Other.EndIndex;
	}

	friend inline uint32 GetTypeHash(const FTextRange& Key)
	{
		uint32 KeyHash = 0;
		KeyHash = HashCombine(KeyHash, GetTypeHash(Key.BeginIndex));
		KeyHash = HashCombine(KeyHash, GetTypeHash(Key.EndIndex));
		return KeyHash;
	}

	int32 Len() const { return EndIndex - BeginIndex; }
	bool IsEmpty() const { return (EndIndex - BeginIndex) <= 0; }
	void Offset(int32 Amount) { BeginIndex += Amount; BeginIndex = FMath::Max(0, BeginIndex);  EndIndex += Amount; EndIndex = FMath::Max(0, EndIndex); }
	bool Contains(int32 Index) const { return Index >= BeginIndex && Index < EndIndex; }
	bool InclusiveContains(int32 Index) const { return Index >= BeginIndex && Index <= EndIndex; }

	FTextRange Intersect(const FTextRange& Other) const
	{
		FTextRange Intersected(FMath::Max(BeginIndex, Other.BeginIndex), FMath::Min(EndIndex, Other.EndIndex));
		if (Intersected.EndIndex <= Intersected.BeginIndex)
		{
			return FTextRange(0, 0);
		}

		return Intersected;
	}

	/**
	 * Produce an array of line ranges from the given text, breaking at any new-line characters
	 */
	static CORE_API void CalculateLineRangesFromString(const FString& Input, TArray<FTextRange>& LineRanges);

	int32 BeginIndex;
	int32 EndIndex;
};

namespace UE::Core::Private
{
	CORE_API void StripNegativeZero(double& InFloat);
}

#include "Misc/StringFormatArg.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Misc/StringOutputDevice.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
