// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Should not be included directly
// IWYU pragma: private

/*******************************************************************************************************************
 * NOTICE                                                                                                          *
 *                                                                                                                 *
 * This file is not intended to be included directly - it will be included by other .h files which have predefined *
 * some macros to be expanded within this file.  As such, it does not have a #pragma once as it is intended to be  *
 * included multiple times with different macro definitions.                                                       *
 *                                                                                                                 *
 * #includes needed to compile this file need to be specified in UnrealStringIncludes.h.inl file rather than here. *
 *******************************************************************************************************************/
#define UE_INCLUDETOOL_IGNORE_INCONSISTENT_STATE

#ifndef UE_STRING_CLASS
	#error "UnrealString.h.inl should only be included after defining UE_STRING_CLASS"
#endif
#ifndef UE_STRING_CHARTYPE
	#error "UnrealString.h.inl should only be included after defining UE_STRING_CHARTYPE"
#endif
#ifndef UE_STRING_CHARTYPE_IS_TCHAR
	#error "UnrealString.h.inl should only be included after defining UE_STRING_CHARTYPE_IS_TCHAR"
#endif
#ifndef UE_STRING_PRINTF_FMT_CHARTYPE
	#error "UnrealString.h.inl should only be included after defining UE_STRING_PRINTF_FMT_CHARTYPE"
#endif
#ifndef UE_STRING_DEPRECATED
	#error "UnrealString.h.inl should only be included after defining UE_STRING_DEPRECATED"
#endif

struct FStringFormatArg;
template<typename InKeyType,typename InValueType,typename SetAllocator ,typename KeyFuncs > class TMap;

using UE_JOIN(UE_STRING_CLASS, FormatNamedArguments) = TMap<UE_STRING_CLASS, FStringFormatArg>;
using FStringFormatOrderedArguments = TArray<FStringFormatArg>;

[[nodiscard]] UE_STRING_CHARTYPE*       GetData(UE_STRING_CLASS&);
[[nodiscard]] const UE_STRING_CHARTYPE* GetData(const UE_STRING_CLASS&);
[[nodiscard]] int32                     GetNum(const UE_STRING_CLASS& String);

#ifdef __OBJC__
@class NSString;
#endif

/**
 * A dynamically sizeable string.
 * @see https://docs.unrealengine.com/latest/INT/Programming/UnrealArchitecture/StringHandling/FString/
 *
 * When dealing with UTF-8 literals, the following advice is recommended:
 *
 * - Do not use the u8"..." prefix (gives the wrong array type until C++20).
 * - Use UTF8TEXT("...") for array literals (type is const UTF8CHAR[n]).
 * - Use UTF8TEXTVIEW("...") for string view literals (type is FUtf8StringView).
 * - Use \uxxxx or \Uxxxxxxxx escape sequences rather than \x to specify Unicode code points.
 */
class UE_STRING_CLASS
{
public:
	using AllocatorType = TSizedDefaultAllocator<32>;
	using ElementType   = UE_STRING_CHARTYPE;
	using FmtCharType   = UE_STRING_PRINTF_FMT_CHARTYPE;

#ifdef UE_STRING_PRINTF_FMT_CHARTYPE2
	// Alternate formatting string type, particularly used for FUtf8String which can accept both "..." and UTF8TEXT("...") for formatting strings
	using FmtCharType2 = UE_STRING_PRINTF_FMT_CHARTYPE2;
#endif

private:
	/** Array holding the character data */
	typedef TArray<ElementType, AllocatorType> DataType;
	DataType Data;

public:

	// Strings can be returned from AutoRTFM::Open() via a copy-construction in the closed.
	static constexpr AutoRTFM::EReturnFromOpenMode AutoRTFMReturnFromOpenMode = AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed;

	[[nodiscard]] UE_STRING_CLASS() = default;
	[[nodiscard]] explicit consteval UE_STRING_CLASS(EConstEval) 
		: Data(ConstEval)
	{
	}
	[[nodiscard]] UE_STRING_CLASS(UE_STRING_CLASS&&) = default;
	[[nodiscard]] UE_STRING_CLASS(const UE_STRING_CLASS&) = default;
	UE_STRING_CLASS& operator=(UE_STRING_CLASS&&) = default;
	UE_STRING_CLASS& operator=(const UE_STRING_CLASS&) = default;

	/**
	 * Create a copy of the Other string with extra space for characters at the end of the string
	 *
	 * @param Other the other string to create a new copy from
	 * @param ExtraSlack number of extra characters to add to the end of the other string in this string
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT UE_STRING_CLASS(const UE_STRING_CLASS& Other, int32 ExtraSlack)
		: Data(Other.Data, ExtraSlack + ((Other.Data.Num() || !ExtraSlack) ? 0 : 1)) // Add 1 if the source string array is empty and we want some slack, because we'll need to include a null terminator which is currently missing
	{
	}

	/**
	 * Create a copy of the Other string with extra space for characters at the end of the string
	 *
	 * @param Other the other string to create a new copy from
	 * @param ExtraSlack number of extra characters to add to the end of the other string in this string
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT UE_STRING_CLASS(UE_STRING_CLASS&& Other, int32 ExtraSlack)
		: Data(MoveTemp(Other.Data), ExtraSlack + ((Other.Data.Num() || !ExtraSlack) ? 0 : 1)) // Add 1 if the source string array is empty and we want some slack, because we'll need to include a null terminator which is currently missing
	{
	}

	/** Construct from null-terminated C string or nullptr  */
	[[nodiscard]] CORE_API UE_STRING_CLASS(const ANSICHAR* Str);
	[[nodiscard]] CORE_API UE_STRING_CLASS(const WIDECHAR* Str);
	[[nodiscard]] CORE_API UE_STRING_CLASS(const UTF8CHAR* Str);
	[[nodiscard]] CORE_API UE_STRING_CLASS(const UCS2CHAR* Str);
	[[nodiscard]] CORE_API UE_STRING_CLASS(const UTF32CHAR* Str);

	/** Construct from null-terminated C substring or nullptr */
	UE_STRING_DEPRECATED(5.4, "This constructor has been deprecated - please use " UE_STRINGIZE(UE_STRING_CLASS) "::ConstructFromPtrSize(Ptr, Size) instead.") [[nodiscard]] CORE_API UE_STRING_CLASS(int32 Len, const ANSICHAR* Str);
	UE_STRING_DEPRECATED(5.4, "This constructor has been deprecated - please use " UE_STRINGIZE(UE_STRING_CLASS) "::ConstructFromPtrSize(Ptr, Size) instead.") [[nodiscard]] CORE_API UE_STRING_CLASS(int32 Len, const WIDECHAR* Str);
	UE_STRING_DEPRECATED(5.4, "This constructor has been deprecated - please use " UE_STRINGIZE(UE_STRING_CLASS) "::ConstructFromPtrSize(Ptr, Size) instead.") [[nodiscard]] CORE_API UE_STRING_CLASS(int32 Len, const UTF8CHAR* Str);
	UE_STRING_DEPRECATED(5.4, "This constructor has been deprecated - please use " UE_STRINGIZE(UE_STRING_CLASS) "::ConstructFromPtrSize(Ptr, Size) instead.") [[nodiscard]] CORE_API UE_STRING_CLASS(int32 Len, const UCS2CHAR* Str);

	/** Construct from null-terminated C string or nullptr with extra slack on top of original string length */
	UE_STRING_DEPRECATED(5.4, "This constructor has been deprecated - please use " UE_STRINGIZE(UE_STRING_CLASS) "::ConstructWithSlack(Ptr, Size) instead.") [[nodiscard]] CORE_API UE_STRING_CLASS(const ANSICHAR* Str, int32 ExtraSlack);
	UE_STRING_DEPRECATED(5.4, "This constructor has been deprecated - please use " UE_STRINGIZE(UE_STRING_CLASS) "::ConstructWithSlack(Ptr, Size) instead.") [[nodiscard]] CORE_API UE_STRING_CLASS(const WIDECHAR* Str, int32 ExtraSlack);
	UE_STRING_DEPRECATED(5.4, "This constructor has been deprecated - please use " UE_STRINGIZE(UE_STRING_CLASS) "::ConstructWithSlack(Ptr, Size) instead.") [[nodiscard]] CORE_API UE_STRING_CLASS(const UTF8CHAR* Str, int32 ExtraSlack);
	UE_STRING_DEPRECATED(5.4, "This constructor has been deprecated - please use " UE_STRINGIZE(UE_STRING_CLASS) "::ConstructWithSlack(Ptr, Size) instead.") [[nodiscard]] CORE_API UE_STRING_CLASS(const UCS2CHAR* Str, int32 ExtraSlack);

	/** Construct from null-terminated C string with extra slack on top of original string length. */
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConstructWithSlack(const ANSICHAR* Str, int32 ExtraSlack);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConstructWithSlack(const WIDECHAR* Str, int32 ExtraSlack);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConstructWithSlack(const UTF8CHAR* Str, int32 ExtraSlack);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConstructWithSlack(const UCS2CHAR* Str, int32 ExtraSlack);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConstructWithSlack(const UTF32CHAR* Str, int32 ExtraSlack);

	/** Construct from a buffer.  If the buffer contains zeros, these will be present in the constructed string. */
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConstructFromPtrSize(const ANSICHAR* Str, int32 Size);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConstructFromPtrSize(const WIDECHAR* Str, int32 Size);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConstructFromPtrSize(const UTF8CHAR* Str, int32 Size);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConstructFromPtrSize(const UCS2CHAR* Str, int32 Size);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConstructFromPtrSize(const UTF32CHAR* Str, int32 Size);

	/** Construct from a buffer with extra slack on top of original string length.  If the buffer contains zeros, these will be present in the constructed string. */
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConstructFromPtrSizeWithSlack(const ANSICHAR* Str, int32 Size, int32 ExtraSlack);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConstructFromPtrSizeWithSlack(const WIDECHAR* Str, int32 Size, int32 ExtraSlack);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConstructFromPtrSizeWithSlack(const UTF8CHAR* Str, int32 Size, int32 ExtraSlack);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConstructFromPtrSizeWithSlack(const UCS2CHAR* Str, int32 Size, int32 ExtraSlack);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConstructFromPtrSizeWithSlack(const UTF32CHAR* Str, int32 Size, int32 ExtraSlack);

	/** Construct from contiguous range of characters such as a string view or string builder */
	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(
			TIsContiguousContainer<CharRangeType>::Value &&
			!std::is_array_v<std::remove_reference_t<CharRangeType>> &&
			TIsCharType_V<CharRangeElementType> &&
			!std::is_base_of_v<UE_STRING_CLASS, std::decay_t<CharRangeType>>
		)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT explicit UE_STRING_CLASS(CharRangeType&& Str)
	{
		*this = UE_STRING_CLASS::ConstructFromPtrSize(GetData(Forward<CharRangeType>(Str)), GetNum(Str));
	}

	/** Construct from contiguous range of characters with extra slack on top of original string length */
	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(
			TIsContiguousContainer<CharRangeType>::Value &&
			!std::is_array_v<std::remove_reference_t<CharRangeType>>&&
			TIsCharType_V<CharRangeElementType> &&
			!std::is_base_of_v<UE_STRING_CLASS, std::decay_t<CharRangeType>>
		)
	>
	[[nodiscard]] explicit UE_STRING_CLASS(CharRangeType&& Str, int32 ExtraSlack)
	{
		uint32 InLen = GetNum(Str);
		Reserve(InLen + ExtraSlack);
		AppendChars(GetData(Forward<CharRangeType>(Str)), InLen);
	}

public:

#if PLATFORM_APPLE && UE_STRING_CHARTYPE_IS_TCHAR
	[[nodiscard]] CORE_API UE_STRING_CLASS(const CFStringRef In);
#endif

	////////////////////////////////////////////////////////
	// Start - intrusive TOptional<UE_STRING_CLASS> state //
	////////////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = UE_STRING_CLASS;

	[[nodiscard]] explicit UE_STRING_CLASS(FIntrusiveUnsetOptionalState Tag)
		: Data(Tag)
	{
	}
	[[nodiscard]] bool UEOpEquals(FIntrusiveUnsetOptionalState Tag) const
	{
		return Data == Tag;
	}
	//////////////////////////////////////////////////////
	// End - intrusive TOptional<UE_STRING_CLASS> state //
	//////////////////////////////////////////////////////

#if defined(__OBJC__) && UE_STRING_CHARTYPE_IS_TCHAR
	/** Convert Objective-C NSString* to string class */
	[[nodiscard]] UE_FORCEINLINE_HINT UE_STRING_CLASS(const NSString* In) : UE_STRING_CLASS((__bridge CFStringRef)In)
	{
	}
#endif

	CORE_API UE_STRING_CLASS& operator=(const ElementType* Str);

	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(
			TIsContiguousContainer<CharRangeType>::Value &&
			!std::is_array_v<std::remove_reference_t<CharRangeType>> &&
			std::is_same_v<ElementType, CharRangeElementType>
		)
	>
	inline UE_STRING_CLASS& operator=(CharRangeType&& Range)
	{
		AssignRange(GetData(Range), GetNum(Range));
		return *this;
	}

private:
	CORE_API void AssignRange(const ElementType* Str, int32 Len);

public:
	/**
	 * Return specific character from this string
	 *
	 * @param Index into string
	 * @return Character at Index
	 */
	[[nodiscard]] inline ElementType& operator[](int32 Index) UE_LIFETIMEBOUND
	{
		checkf(IsValidIndex(Index), TEXT("String index out of bounds: Index %i from a string with a length of %i"), Index, Len());
		return Data.GetData()[Index];
	}

	/**
	 * Return specific const character from this string
	 *
	 * @param Index into string
	 * @return const Character at Index
	 */
	[[nodiscard]] inline const ElementType& operator[](int32 Index) const UE_LIFETIMEBOUND
	{
		checkf(IsValidIndex(Index), TEXT("String index out of bounds: Index %i from a string with a length of %i"), Index, Len());
		return Data.GetData()[Index];
	}

	/**
	 * Iterator typedefs
	 */
	typedef DataType::TIterator      TIterator;
	typedef DataType::TConstIterator TConstIterator;

	/** Creates an iterator for the characters in this string */
	[[nodiscard]] UE_FORCEINLINE_HINT TIterator CreateIterator()
	{
		return Data.CreateIterator();
	}

	/** Creates a const iterator for the characters in this string */
	[[nodiscard]] UE_FORCEINLINE_HINT TConstIterator CreateConstIterator() const
	{
		return Data.CreateConstIterator();
	}

	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT DataType::RangedForIteratorType             begin ()       { auto Result = Data.begin();                                return Result; }
	[[nodiscard]] UE_FORCEINLINE_HINT DataType::RangedForConstIteratorType        begin () const { auto Result = Data.begin();                                return Result; }
	[[nodiscard]] UE_FORCEINLINE_HINT DataType::RangedForIteratorType             end   ()       { auto Result = Data.end();    if (Data.Num()) { --Result; } return Result; }
	[[nodiscard]] UE_FORCEINLINE_HINT DataType::RangedForConstIteratorType        end   () const { auto Result = Data.end();    if (Data.Num()) { --Result; } return Result; }
	[[nodiscard]] UE_FORCEINLINE_HINT DataType::RangedForReverseIteratorType      rbegin()       { auto Result = Data.rbegin(); if (Data.Num()) { ++Result; } return Result; }
	[[nodiscard]] UE_FORCEINLINE_HINT DataType::RangedForConstReverseIteratorType rbegin() const { auto Result = Data.rbegin(); if (Data.Num()) { ++Result; } return Result; }
	[[nodiscard]] UE_FORCEINLINE_HINT DataType::RangedForReverseIteratorType      rend  ()       { auto Result = Data.rend();                                 return Result; }
	[[nodiscard]] UE_FORCEINLINE_HINT DataType::RangedForConstReverseIteratorType rend  () const { auto Result = Data.rend();                                 return Result; }

	[[nodiscard]] UE_FORCEINLINE_HINT SIZE_T GetAllocatedSize() const
	{
		return Data.GetAllocatedSize();
	}

	/**
	 * Run slow checks on this string
	 */
	inline void CheckInvariants() const
	{
		int32 Num = Data.Num();
		checkSlow(Num >= 0);
		checkSlow(!Num || !Data.GetData()[Num - 1]);
		checkSlow(Data.GetSlack() >= 0);
	}

	/**
	 * Create empty string of given size with zero terminating character
	 *
	 * @param Slack length of empty string to create
	 */
	CORE_API void Empty();
	CORE_API void Empty(int32 Slack);

	/**
	 * Test whether this string is empty
	 *
	 * @return true if this string is empty, otherwise return false.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return Data.Num() <= 1;
	}

	/**
	 * Empties the string, but doesn't change memory allocation, unless the new size is larger than the current string.
	 *
	 * @param NewReservedSize The expected usage size (in characters, not including the terminator) after calling this function.
	 */
	CORE_API void Reset(int32 NewReservedSize = 0);

	/**
	 * Remove unallocated empty character space from the end of this string
	 */
	CORE_API void Shrink();

	/**
	 * Tests if index is valid, i.e. greater than or equal to zero, and less than the number of characters in this string (excluding the null terminator).
	 *
	 * @param Index Index to test.
	 *
	 * @returns True if index is valid. False otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValidIndex(int32 Index) const
	{
		return Index >= 0 && Index < Len();
	}

	/**
	 * Get pointer to the string
	 *
	 * @Return Pointer to Array of ElementType if Num, otherwise the empty string
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType* operator*() const UE_LIFETIMEBOUND
	{
		return Data.Num() ? Data.GetData() : CHARTEXT(ElementType, "");
	}

	/** 
	 *Get string as array of TCHARS 
	 *
	 * @warning: Operations on the TArray<*CHAR> can be unsafe, such as adding
	 *		non-terminating 0's or removing the terminating zero.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT DataType& GetCharArray() UE_LIFETIMEBOUND
	{
		return Data;
	}

	/** Get string as const array of TCHARS */
	[[nodiscard]] UE_FORCEINLINE_HINT const DataType& GetCharArray() const UE_LIFETIMEBOUND
	{
		return Data;
	}

#if PLATFORM_APPLE
	/** Convert the string to C bridgable CFString */
	[[nodiscard]] CORE_API CFStringRef GetCFString() const;
#endif

#ifdef __OBJC__
	/** Convert the string to Objective-C NSString */
	[[nodiscard]] CORE_API NSString* GetNSString() const;
#endif
	/**
	 * Appends a character range without null-terminators in it
	 *
	 * @param Str can be null if Count is 0. Can be unterminated, Str[Count] isn't read.
	 */
	CORE_API void AppendChars(const ANSICHAR* Str, int32 Count);
	CORE_API void AppendChars(const WIDECHAR* Str, int32 Count);
	CORE_API void AppendChars(const UCS2CHAR* Str, int32 Count);
	CORE_API void AppendChars(const UTF8CHAR* Str, int32 Count);
	CORE_API void AppendChars(const UTF32CHAR* Str, int32 Count);

	/** Append a string and return a reference to this */
	template<class CharType>
	inline UE_STRING_CLASS& Append(const CharType* Str, int32 Count)
	{
		AppendChars(Str, Count);
		return *this;
	}
	
	/**
	 * Append a valid null-terminated string and return a reference to this
	 *
	 * CharType is not const to use this overload for mutable char arrays and call
	 * Strlen() instead of getting the static length N from GetNum((&T)[N]).
	 * Oddly MSVC ranks a const T* overload over T&& for T[N] while clang picks T&&.
	 */
	template<class CharType>
	inline UE_STRING_CLASS& Append(/* no const! */ CharType* Str)
	{
		checkSlow(Str);
		AppendChars(Str, TCString<std::remove_const_t<CharType>>::Strlen(Str));
		return *this;
	}

	/** Append a string and return a reference to this */
	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(TIsContiguousContainer<CharRangeType>::Value && TIsCharType_V<CharRangeElementType>)
	>
	inline UE_STRING_CLASS& Append(CharRangeType&& Str)
	{
		AppendChars(GetData(Str), GetNum(Str));
		return *this;
	}

	/** Append a single character and return a reference to this */
	CORE_API UE_STRING_CLASS& AppendChar(ElementType InChar);

	/** Append a string and return a reference to this */
	template <typename StrType>
	UE_FORCEINLINE_HINT auto operator+=(StrType&& Str) -> decltype(Append(Forward<StrType>(Str)))
	{
		return Append(Forward<StrType>(Str));
	}

	/** Append a single character and return a reference to this */
	template <
		typename AppendedCharType
		UE_REQUIRES(TIsCharType_V<AppendedCharType>)
	>
	inline UE_STRING_CLASS& operator+=(AppendedCharType Char)
	{
		if constexpr (TIsCharEncodingSimplyConvertibleTo_V<AppendedCharType, ElementType>)
		{
			return AppendChar((ElementType)Char);
		}
		else
		{
			AppendChars(&Char, 1);
			return *this;
		}
	}

	CORE_API void InsertAt(int32 Index, ElementType Character);
	CORE_API void InsertAt(int32 Index, const UE_STRING_CLASS& Characters);

	/**
	 * Removes a character from the string.
	 *
	 * @param Index          The index of the character to remove.
	 * @param AllowShrinking Whether or not to reallocate to shrink the storage after removal.
	 */
	CORE_API void RemoveAt(int32 Index, EAllowShrinking AllowShrinking = EAllowShrinking::Default);

	/**
	 * Removes characters within the string.
	 *
	 * @param Index          The index of the first character to remove.
	 * @param Count          The number of characters to remove.
	 * @param AllowShrinking Whether or not to reallocate to shrink the storage after removal.
	 */
	CORE_API void RemoveAt(int32 Index, int32 Count, EAllowShrinking AllowShrinking = EAllowShrinking::Default);

	/**
	 * Removes the text from the start of the string if it exists.
	 *
	 * @param InPrefix    The prefix to search for at the start of the string to remove.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if the prefix was removed, otherwise false.
	 */
	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(
			TIsContiguousContainer<CharRangeType>::Value &&
			!std::is_array_v<std::remove_reference_t<CharRangeType>>&&
			TIsCharType_V<CharRangeElementType> &&
			!std::is_base_of_v<UE_STRING_CLASS, std::decay_t<CharRangeType>>
		)
	>
	bool RemoveFromStart(CharRangeType&& InPrefix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
	{
		static_assert(std::is_same_v<CharRangeElementType, ElementType>, "Expected a range of ElementType");
		return RemoveFromStart(GetData(InPrefix), GetNum(InPrefix), SearchCase);
	}

	/**
	 * Removes the text from the start of the string if it exists.
	 *
	 * @param InPrefix    The prefix to search for at the start of the string to remove.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if the prefix was removed, otherwise false.
	 */
	bool RemoveFromStart(const ElementType* InPrefix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
	{
		return RemoveFromStart(InPrefix, InPrefix ? TCString<ElementType>::Strlen(InPrefix) : 0, SearchCase);
	}

	/**
	 * Removes the text from the start of the string if it exists.
	 *
	 * @param InPrefix    The prefix to search for at the start of the string to remove.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if the prefix was removed, otherwise false.
	 */
	bool RemoveFromStart(const UE_STRING_CLASS& InPrefix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
	{
		return RemoveFromStart(*InPrefix, InPrefix.Len(), SearchCase);
	}

	/**
	 * Removes the text from the start of the string if it exists.
	 *
	 * @param InPrefix     The prefix to search for at the start of the string to remove.
	 * @param InPrefixLen  The length of InPrefix
	 * @return true if the prefix was removed, otherwise false.
	 */
	CORE_API bool RemoveFromStart(const ElementType* InPrefix, int32 InPrefixLen, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase);

	/**
	 * Removes the text from the end of the string if it exists.
	 *
	 * @param InSuffix    The suffix to search for at the end of the string to remove.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if the suffix was removed, otherwise false.
	 */
	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(
			TIsContiguousContainer<CharRangeType>::Value &&
			!std::is_array_v<std::remove_reference_t<CharRangeType>>&&
			TIsCharType_V<CharRangeElementType> &&
			!std::is_base_of_v<UE_STRING_CLASS, std::decay_t<CharRangeType>>
		)
	>
	bool RemoveFromEnd(CharRangeType&& InSuffix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
	{
		static_assert(std::is_same_v<CharRangeElementType, ElementType>, "Expected a range of ElementType");
		return RemoveFromEnd(GetData(InSuffix), GetNum(InSuffix), SearchCase);
	}

	/**
	 * Removes the text from the end of the string if it exists.
	 *
	 * @param InSuffix    The suffix to search for at the end of the string to remove.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if the suffix was removed, otherwise false.
	 */
	bool RemoveFromEnd(const ElementType* InSuffix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
	{
		return RemoveFromEnd(InSuffix, InSuffix ? TCString<ElementType>::Strlen(InSuffix) : 0, SearchCase);
	}

	/**
	 * Removes the text from the end of the string if it exists.
	 *
	 * @param InSuffix    The suffix to search for at the end of the string to remove.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if the suffix was removed, otherwise false.
	 */
	bool RemoveFromEnd(const UE_STRING_CLASS& InSuffix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
	{
		return RemoveFromEnd(*InSuffix, InSuffix.Len(), SearchCase);
	}

	/**
	 * Removes the text from the end of the string if it exists.
	 *
	 * @param InSuffix     The suffix to search for at the end of the string to remove.
	 * @param InSuffixLen  The length of InSuffix
	 * @param SearchCase   Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if the suffix was removed, otherwise false.
	 */
	CORE_API bool RemoveFromEnd(const ElementType* InSuffix, int32 InSuffixLen, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase);

	/**
	 * Concatenate this path with given path ensuring the / character is used between them
	 *
	 * @param Str       Pointer to an array of TCHARs (not necessarily null-terminated) to be concatenated onto the end of this.
	 * @param StrLength Exact number of characters from Str to append.
	 */
	CORE_API void PathAppend(const ElementType* Str, int32 StrLength);

	/**
	 * Concatenates a string with a character.
	 * 
	 * @param Lhs The string on the left-hand-side of the expression.
	 * @param Rhs The char on the right-hand-side of the expression.
	 *
	 * @return The concatenated string.
	 */
	template <
		typename CharType
		UE_REQUIRES(TIsCharType_V<CharType>)
	>
	[[nodiscard]] inline friend UE_STRING_CLASS operator+(const UE_STRING_CLASS& Lhs, CharType Rhs)
	{
		Lhs.CheckInvariants();

		UE_STRING_CLASS Result(Lhs, 1);
		Result += Rhs;

		return Result;
	}

	/**
	 * Concatenates a string with a character.
	 * 
	 * @param Lhs The string on the left-hand-side of the expression.
	 * @param Rhs The char on the right-hand-side of the expression.
	 *
	 * @return The concatenated string.
	 */
	template <
		typename CharType
		UE_REQUIRES(TIsCharType_V<CharType>)
	>
	[[nodiscard]] inline friend UE_STRING_CLASS operator+(UE_STRING_CLASS&& Lhs, CharType Rhs)
	{
		Lhs.CheckInvariants();

		UE_STRING_CLASS Result(MoveTemp(Lhs), 1);
		Result += Rhs;

		return Result;
	}

private:
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConcatFF(const UE_STRING_CLASS& Lhs, const UE_STRING_CLASS& Rhs);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConcatFF(UE_STRING_CLASS&& Lhs, const UE_STRING_CLASS& Rhs);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConcatFF(const UE_STRING_CLASS& Lhs, UE_STRING_CLASS&& Rhs);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConcatFF(UE_STRING_CLASS&& Lhs, UE_STRING_CLASS&& Rhs);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConcatFC(const UE_STRING_CLASS& Lhs, const ElementType* Rhs);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConcatFC(UE_STRING_CLASS&& Lhs,	const ElementType* Rhs);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConcatCF(const ElementType* Lhs, const UE_STRING_CLASS& Rhs);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConcatCF(const ElementType* Lhs, UE_STRING_CLASS&& Rhs);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConcatFR(const UE_STRING_CLASS& Lhs, const ElementType* Rhs, int32 RhsLen);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConcatFR(UE_STRING_CLASS&& Lhs,	const ElementType* Rhs, int32 RhsLen);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConcatRF(const ElementType* Lhs, int32 LhsLen, const UE_STRING_CLASS& Rhs);
	[[nodiscard]] static CORE_API UE_STRING_CLASS ConcatRF(const ElementType* Lhs, int32 LhsLen, UE_STRING_CLASS&& Rhs);

public:
	[[nodiscard]] UE_FORCEINLINE_HINT friend UE_STRING_CLASS operator+(const UE_STRING_CLASS& Lhs, const UE_STRING_CLASS& Rhs)	{ return ConcatFF(Lhs, Rhs); }
	[[nodiscard]] UE_FORCEINLINE_HINT friend UE_STRING_CLASS operator+(UE_STRING_CLASS&& Lhs, const UE_STRING_CLASS& Rhs)		{ return ConcatFF(MoveTemp(Lhs), Rhs); }
	[[nodiscard]] UE_FORCEINLINE_HINT friend UE_STRING_CLASS operator+(const UE_STRING_CLASS& Lhs, UE_STRING_CLASS&& Rhs)		{ return ConcatFF(Lhs,MoveTemp(Rhs)); }
	[[nodiscard]] UE_FORCEINLINE_HINT friend UE_STRING_CLASS operator+(UE_STRING_CLASS&& Lhs, UE_STRING_CLASS&& Rhs)				{ return ConcatFF(MoveTemp(Lhs), MoveTemp(Rhs)); }
	[[nodiscard]] UE_FORCEINLINE_HINT friend UE_STRING_CLASS operator+(const ElementType* Lhs, const UE_STRING_CLASS& Rhs)		{ return ConcatCF(Lhs, Rhs); }
	[[nodiscard]] UE_FORCEINLINE_HINT friend UE_STRING_CLASS operator+(const ElementType* Lhs, UE_STRING_CLASS&& Rhs)			{ return ConcatCF(Lhs, MoveTemp(Rhs)); }
	[[nodiscard]] UE_FORCEINLINE_HINT friend UE_STRING_CLASS operator+(const UE_STRING_CLASS& Lhs, const ElementType* Rhs)		{ return ConcatFC(Lhs, Rhs); }
	[[nodiscard]] UE_FORCEINLINE_HINT friend UE_STRING_CLASS operator+(UE_STRING_CLASS&& Lhs, const ElementType* Rhs)			{ return ConcatFC(MoveTemp(Lhs), Rhs); }

	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(
			TIsContiguousContainer<CharRangeType>::Value &&
			!std::is_array_v<std::remove_reference_t<CharRangeType>> &&
			std::is_same_v<ElementType, CharRangeElementType> &&
			!std::is_base_of_v<UE_STRING_CLASS, std::decay_t<CharRangeType>>
		)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT friend UE_STRING_CLASS operator+(CharRangeType&& Lhs, const UE_STRING_CLASS& Rhs)
	{
		return ConcatRF(GetData(Lhs), GetNum(Lhs), Rhs);
	}
	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(
			TIsContiguousContainer<CharRangeType>::Value &&
			!std::is_array_v<std::remove_reference_t<CharRangeType>>&&
			std::is_same_v<ElementType, CharRangeElementType> &&
			!std::is_base_of_v<UE_STRING_CLASS, std::decay_t<CharRangeType>>
		)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT friend UE_STRING_CLASS operator+(CharRangeType&& Lhs, UE_STRING_CLASS&& Rhs)
	{
		return ConcatRF(GetData(Lhs), GetNum(Lhs), MoveTemp(Rhs));
	}
	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(
			TIsContiguousContainer<CharRangeType>::Value &&
			!std::is_array_v<std::remove_reference_t<CharRangeType>>&&
			std::is_same_v<ElementType, CharRangeElementType> &&
			!std::is_base_of_v<UE_STRING_CLASS, std::decay_t<CharRangeType>>
		)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT friend UE_STRING_CLASS operator+(const UE_STRING_CLASS& Lhs, CharRangeType&& Rhs)
	{
		return ConcatFR(Lhs, GetData(Rhs), GetNum(Rhs));
	}
	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(
			TIsContiguousContainer<CharRangeType>::Value &&
			!std::is_array_v<std::remove_reference_t<CharRangeType>>&&
			std::is_same_v<ElementType, CharRangeElementType> &&
			!std::is_base_of_v<UE_STRING_CLASS, std::decay_t<CharRangeType>>
		)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT friend UE_STRING_CLASS operator+(UE_STRING_CLASS&& Lhs, CharRangeType&& Rhs)
	{
		return ConcatFR(MoveTemp(Lhs), GetData(Rhs), GetNum(Rhs));
	}

	/**
	 * Concatenate this path with given path ensuring the / character is used between them
	 * 
	 * @param Str path array of characters to be concatenated onto the end of this
	 * @return reference to path
	 */
	inline UE_STRING_CLASS& operator/=(const ElementType* Str)
	{
		checkSlow(Str);

		PathAppend(Str, TCString<ElementType>::Strlen(Str));
		return *this;
	}

	/**
	* Concatenate this path with given path ensuring the / character is used between them
	* 
	* @param Str path range of CharType (string class/string view/string builder etc) to be concatenated onto the end of this
	* @return reference to path
	*/
	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(
			TIsContiguousContainer<CharRangeType>::Value &&
			!std::is_array_v<std::remove_reference_t<CharRangeType>>&&
			std::is_same_v<ElementType, CharRangeElementType>
		)
	>
	inline UE_STRING_CLASS& operator/=(CharRangeType&& Str)
	{
		PathAppend(GetData(Str), GetNum(Str));
		return *this;
	}

	/**
	* Concatenate this path with given path ensuring the / character is used between them
	* 
	* @param Str path array of CharType (that needs converting) to be concatenated onto the end of this
	* @return reference to path
	*/
	template <
		typename CharType
		UE_REQUIRES(TIsCharType_V<CharType>)
	>
	inline UE_STRING_CLASS& operator/=(const CharType* Str)
	{
		UE_STRING_CLASS Temp = Str;
		PathAppend(GetData(Temp), GetNum(Temp));
		return *this;
	}

	/**
	 * Concatenate this path with given path ensuring the / character is used between them
	 *
	 * @param Lhs Path to concatenate onto.
	 * @param Rhs Path to concatenate.
	 * @return The new concatenated path
	 */
	[[nodiscard]] inline friend UE_STRING_CLASS operator/(const UE_STRING_CLASS& Lhs, const ElementType* Rhs)
	{
		checkSlow(Rhs);

		int32 StrLength = TCString<ElementType>::Strlen(Rhs);

		UE_STRING_CLASS Result(Lhs, StrLength + 1);
		Result.PathAppend(Rhs, StrLength);
		return Result;
	}

	/**
	 * Concatenate this path with given path ensuring the / character is used between them
	 *
	 * @param Lhs Path to concatenate onto.
	 * @param Rhs Path to concatenate.
	 * @return The new concatenated path
	 */
	[[nodiscard]] inline friend UE_STRING_CLASS operator/(UE_STRING_CLASS&& Lhs, const ElementType* Rhs)
	{
		checkSlow(Rhs);

		int32 StrLength = TCString<ElementType>::Strlen(Rhs);

		UE_STRING_CLASS Result(MoveTemp(Lhs), StrLength + 1);
		Result.PathAppend(Rhs, StrLength);
		return Result;
	}

	/**
	 * Concatenate this path with given path ensuring the / character is used between them
	 *
	 * @param Lhs Path to concatenate onto.
	 * @param Rhs Path to concatenate.
	 * @return The new concatenated path
	 */
	[[nodiscard]] inline friend UE_STRING_CLASS operator/(const UE_STRING_CLASS& Lhs, const UE_STRING_CLASS& Rhs)
	{
		int32 StrLength = Rhs.Len();

		UE_STRING_CLASS Result(Lhs, StrLength + 1);
		Result.PathAppend(Rhs.Data.GetData(), StrLength);
		return Result;
	}

	/**
	 * Concatenate this path with given path ensuring the / character is used between them
	 *
	 * @param Lhs Path to concatenate onto.
	 * @param Rhs Path to concatenate.
	 * @return The new concatenated path
	 */
	[[nodiscard]] inline friend UE_STRING_CLASS operator/(UE_STRING_CLASS&& Lhs, const UE_STRING_CLASS& Rhs)
	{
		int32 StrLength = Rhs.Len();

		UE_STRING_CLASS Result(MoveTemp(Lhs), StrLength + 1);
		Result.PathAppend(Rhs.Data.GetData(), StrLength);
		return Result;
	}

	/**
	 * Concatenate this path with given path ensuring the / character is used between them
	 *
	 * @param Lhs Path to concatenate onto.
	 * @param Rhs Path to concatenate.
	 * @return new string of the path
	 */
	[[nodiscard]] inline friend UE_STRING_CLASS operator/(const ElementType* Lhs, const UE_STRING_CLASS& Rhs)
	{
		int32 StrLength = Rhs.Len();

		UE_STRING_CLASS Result(UE_STRING_CLASS(Lhs), StrLength + 1);
		Result.PathAppend(Rhs.Data.GetData(), Rhs.Len());
		return Result;
	}

	/**
	 * Lexicographically test whether the left string is < the right string
	 *
	 * @param Lhs String to compare against.
	 * @param Rhs String to compare against.
	 * @return true if the left string is lexicographically < the right string, otherwise false
	 * @note case insensitive
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool UEOpLessThan(const UE_STRING_CLASS& Rhs) const
	{
		return FPlatformString::Stricmp(**this, *Rhs) < 0;
	}

	/**
	 * Lexicographically test whether the left string is < the right string
	 *
	 * @param Lhs String to compare against.
	 * @param Rhs String to compare against.
	 * @return true if the left string is lexicographically < the right string, otherwise false
	 * @note case insensitive
	 */
	template <typename CharType>
	[[nodiscard]] UE_FORCEINLINE_HINT bool UEOpLessThan(const CharType* Rhs) const
	{
		return FPlatformString::Stricmp(**this, Rhs) < 0;
	}

	/**
	 * Lexicographically test whether the left string is > the right string
	 *
	 * @param Lhs String to compare against.
	 * @param Rhs String to compare against.
	 * @return true if the left string is lexicographically > the right string, otherwise false
	 * @note case insensitive
	 */
	template <typename CharType>
	[[nodiscard]] UE_FORCEINLINE_HINT bool UEOpGreaterThan(const CharType* Rhs) const
	{
		return FPlatformString::Stricmp(**this, Rhs) > 0;
	}

	/**
	 * Lexicographically test whether the left string is == the right string
	 *
	 * @param Lhs String to compare against.
	 * @param Rhs String to compare against.
	 * @return true if the left string is lexicographically == the right string, otherwise false
	 * @note case insensitive
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool UEOpEquals(const UE_STRING_CLASS& Rhs) const
	{
		return Equals(Rhs, ESearchCase::IgnoreCase);
	}

	/**
	 * Lexicographically test whether the left string is == the right string
	 *
	 * @param Lhs String to compare against.
	 * @param Rhs String to compare against.
	 * @return true if the left string is lexicographically == the right string, otherwise false
	 * @note case insensitive
	 */
	template <typename CharType>
	[[nodiscard]] UE_FORCEINLINE_HINT bool UEOpEquals(const CharType* Rhs) const
	{
		return FPlatformString::Stricmp(**this, Rhs) == 0;
	}

	/** Get the length of the string, excluding terminating character */
	[[nodiscard]] UE_FORCEINLINE_HINT int32 Len() const
	{
		return Data.Num() ? Data.Num() - 1 : 0;
	}

	/** @returns Number of bytes used for characters, excluding the null-terminator and slack */
	[[nodiscard]] UE_FORCEINLINE_HINT SIZE_T NumBytesWithoutNull() const
	{
		return static_cast<SIZE_T>(Len()) * sizeof(ElementType);
	}

	/** @returns Number of bytes used for characters, including the null-terminator but excluding slack */
	[[nodiscard]] UE_FORCEINLINE_HINT SIZE_T NumBytesWithNull() const
	{
		return (static_cast<SIZE_T>(Len()) + 1) * sizeof(ElementType);
	}

	/** Returns the left most given number of characters */
	[[nodiscard]] UE_FORCEINLINE_HINT UE_STRING_CLASS Left(int32 Count) const &
	{
		return UE_STRING_CLASS::ConstructFromPtrSize(**this, FMath::Clamp(Count, 0, Len()));
	}

	[[nodiscard]] inline UE_STRING_CLASS Left(int32 Count) &&
	{
		LeftInline(Count, EAllowShrinking::No);
		return MoveTemp(*this);
	}

	/** Modifies the string such that it is now the left most given number of characters */
	inline void LeftInline(int32 Count, EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		const int32 Length = Len();
		Count = FMath::Clamp(Count, 0, Length);
		RemoveAt(Count, Length-Count, AllowShrinking);
	}

	/** Returns the left most characters from the string chopping the given number of characters from the end */
	[[nodiscard]] inline UE_STRING_CLASS LeftChop(int32 Count) const &
	{
		const int32 Length = Len();
		return UE_STRING_CLASS::ConstructFromPtrSize(**this, FMath::Clamp(Length - Count, 0, Length));
	}

	[[nodiscard]] inline UE_STRING_CLASS LeftChop(int32 Count) &&
	{
		LeftChopInline(Count, EAllowShrinking::No);
		return MoveTemp(*this);
	}

	/** Modifies the string such that it is now the left most characters chopping the given number of characters from the end */
	inline void LeftChopInline(int32 Count, EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		const int32 Length = Len();
		RemoveAt(FMath::Clamp(Length-Count, 0, Length), Count, AllowShrinking);
	}

	/** Returns the string to the right of the specified location, counting back from the right (end of the word). */
	[[nodiscard]] inline UE_STRING_CLASS Right(int32 Count) const &
	{
		const int32 Length = Len();
		return UE_STRING_CLASS(**this + Length - FMath::Clamp(Count, 0, Length));
	}

	[[nodiscard]] inline UE_STRING_CLASS Right(int32 Count) &&
	{
		RightInline(Count, EAllowShrinking::No);
		return MoveTemp(*this);
	}

	/** Modifies the string such that it is now the right most given number of characters */
	inline void RightInline(int32 Count, EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		const int32 Length = Len();
		RemoveAt(0, Length-FMath::Clamp(Count,0,Length), AllowShrinking);
	}

	/** Returns the string to the right of the specified location, counting forward from the left (from the beginning of the word). */
	[[nodiscard]] CORE_API UE_STRING_CLASS RightChop(int32 Count) const &;

	[[nodiscard]] inline UE_STRING_CLASS RightChop(int32 Count) &&
	{
		RightChopInline(Count, EAllowShrinking::No);
		return MoveTemp(*this);
	}

	/** Modifies the string such that it is now the string to the right of the specified location, counting forward from the left (from the beginning of the word). */
	UE_FORCEINLINE_HINT void RightChopInline(int32 Count, EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		RemoveAt(0, Count, AllowShrinking);
	}

	/** Returns the substring from Start position for Count characters. */
	[[nodiscard]] CORE_API UE_STRING_CLASS Mid(int32 Start, int32 Count) const &;
	[[nodiscard]] CORE_API UE_STRING_CLASS Mid(int32 Start, int32 Count) &&;

	/** Returns the substring from Start position to the end */
	[[nodiscard]] UE_FORCEINLINE_HINT UE_STRING_CLASS Mid(int32 Start) const & { return RightChop(Start); }
	[[nodiscard]] UE_FORCEINLINE_HINT UE_STRING_CLASS Mid(int32 Start) && { return ((UE_STRING_CLASS&&)*this).RightChop(Start); }

	/** Modifies the string such that it is now the substring from Start position for Count characters. */
	inline void MidInline(int32 Start, int32 Count = MAX_int32, EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		if (Count != MAX_int32 && int64(Start) + Count < MAX_int32)
		{
			LeftInline(Count + Start, EAllowShrinking::No);
		}
		RightChopInline(Start, AllowShrinking);
	}

	/**
	 * Searches the string for a substring. Can search from beginning or end, and ignore case or not.
	 *
	 * @param SubStr			The string to search for
	 * @param SearchCase		Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @param SearchDir			Indicates whether the search starts at the beginning or at the end (defaults to ESearchDir::FromStart)
	 * @param StartPosition		The start character position to search from
	 *
	 * @return  Index into this string of the first found instance. If SubStr is empty, returns clamped StartPosition.
	 *
	 * @note  When searching backwards where a StartPosition is provided, searching will actually begin from
	 *        StartPosition - GetNum(SubStr), therefore:
	 *
	 *        FString("X").Find("X", ESearchCase::CaseSensitive, ESearchDir::FromEnd, 0) == INDEX_NONE
	 *
	 *        Consider using UE::String::FindLast() as an alternative.
	 */
	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(
			TIsContiguousContainer<CharRangeType>::Value &&
			!std::is_array_v<std::remove_reference_t<CharRangeType>>&&
			TIsCharType_V<CharRangeElementType> &&
			!std::is_base_of_v<UE_STRING_CLASS, std::decay_t<CharRangeType>>
		)
	>
	[[nodiscard]] int32 Find(CharRangeType&& SubStr, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase,
		ESearchDir::Type SearchDir = ESearchDir::FromStart, int32 StartPosition = INDEX_NONE) const
	{
		static_assert(std::is_same_v<CharRangeElementType, ElementType>, "Expected a range of ElementType");
		return Find(GetData(SubStr), GetNum(SubStr), SearchCase, SearchDir, StartPosition);
	}

	/**
	 * Searches the string for a substring. Can search from beginning or end, and ignore case or not.
	 *
	 * @param SubStr			The string array of characters to search for
	 * @param SearchCase		Indicates whether the search is case sensitive or not
	 * @param SearchDir			Indicates whether the search starts at the beginning or at the end.
	 * @param StartPosition		The start character position to search from.  See note below.
	 *
	 * @return  Index into this string of the first found instance. If SubStr is empty, returns clamped StartPosition.
	 *
	 * @note  When searching backwards where a StartPosition is provided, searching will actually begin from
	 *        StartPosition - Strlen(SubStr), therefore:
	 *
	 *        FString("X").Find("X", ESearchCase::CaseSensitive, ESearchDir::FromEnd, 0) == INDEX_NONE
	 *
	 *        Consider using UE::String::FindLast() as an alternative.
	 */
	[[nodiscard]] int32 Find(const ElementType* SubStr, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase,
		ESearchDir::Type SearchDir = ESearchDir::FromStart, int32 StartPosition = INDEX_NONE) const
	{
		return SubStr ? Find(SubStr, TCString<ElementType>::Strlen(SubStr), SearchCase, SearchDir, StartPosition) : INDEX_NONE;
	}

	/**
	 * Searches the string for a substring. Can search from beginning or end, and ignore case or not.
	 *
	 * @param SubStr			The string array of characters to search for
	 * @param StartPosition		The start character position to search from.  See note below.
	 * @param SearchCase		Indicates whether the search is case sensitive or not
	 * @param SearchDir			Indicates whether the search starts at the beginning or at the end.
	 *
	 * @return  Index into this string of the first found instance. If SubStr is empty, returns clamped StartPosition.
	 *
	 * @note  When searching backwards where a StartPosition is provided, searching will actually begin from
	 *        StartPosition - SubStr.Len(), therefore:
	 *
	 *        FString("X").Find("X", ESearchCase::CaseSensitive, ESearchDir::FromEnd, 0) == INDEX_NONE
	 *
	 *        Consider using UE::String::FindLast() as an alternative.
	 */
	[[nodiscard]] int32 Find(const UE_STRING_CLASS& SubStr, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase,
		ESearchDir::Type SearchDir = ESearchDir::FromStart, int32 StartPosition = INDEX_NONE) const
	{
		return Find(*SubStr, SubStr.Len(), SearchCase, SearchDir, StartPosition);
	}

	/**
	 * Searches the string for a substring. Can search from beginning or end, and ignore case or not.
	 *
	 * @param SubStr			The string array of characters to search for
	 * @param SubStrLen			The length of the SubStr array
	 * @param SearchCase		Indicates whether the search is case sensitive or not
	 * @param SearchDir			Indicates whether the search starts at the beginning or at the end.
	 * @param StartPosition		The start character position to search from.  See note below.
	 *
	 * @return  Index into this string of the first found instance. If SubStr is empty, returns clamped StartPosition.
	 *
	 * @note  When searching backwards where a StartPosition is provided, searching will actually begin from
	 *        StartPosition - InSubStrLen, therefore:
	 *
	 *        FString("X").Find("X", ESearchCase::CaseSensitive, ESearchDir::FromEnd, 0) == INDEX_NONE
	 *
	 *        Consider using UE::String::FindLast() as an alternative.
	 */
	[[nodiscard]] CORE_API int32 Find(const ElementType* SubStr, int32 InSubStrLen, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase,
		ESearchDir::Type SearchDir = ESearchDir::FromStart, int32 StartPosition = INDEX_NONE) const;


	/**
	 * Returns whether this string contains the specified substring.
	 *
	 * @param SubStr			Text to search for
	 * @param SearchCase		Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @param SearchDir			Indicates whether the search starts at the beginning or at the end (defaults to ESearchDir::FromStart)
	 * @return					Returns whether the string contains the substring. If the substring is empty, returns true.
	 **/
	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(
			TIsContiguousContainer<CharRangeType>::Value &&
			!std::is_array_v<std::remove_reference_t<CharRangeType>>&&
			TIsCharType_V<CharRangeElementType> &&
			!std::is_base_of_v<UE_STRING_CLASS, std::decay_t<CharRangeType>>
		)
	>
	[[nodiscard]] inline bool Contains(CharRangeType&& SubStr, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase,
		ESearchDir::Type SearchDir = ESearchDir::FromStart) const
	{
		static_assert(std::is_same_v<CharRangeElementType, ElementType>, "Expected a range of characters");
		return Find(Forward<CharRangeType>(SubStr), SearchCase, SearchDir) != INDEX_NONE;
	}

	/** 
	 * Returns whether this string contains the specified substring.
	 *
	 * @param SubStr			Text to search for
	 * @param SearchCase		Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @param SearchDir			Indicates whether the search starts at the beginning or at the end (defaults to ESearchDir::FromStart)
	 * @return					Returns whether the string contains the substring. If the substring is empty, returns true.
	 **/
	[[nodiscard]] UE_FORCEINLINE_HINT bool Contains(const ElementType* SubStr, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase,
		ESearchDir::Type SearchDir = ESearchDir::FromStart) const
	{
		return Find(SubStr, SearchCase, SearchDir) != INDEX_NONE;
	}

	/** 
	 * Returns whether this string contains the specified substring.
	 *
	 * @param SubStr			Text to search for
	 * @param SearchCase		Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @param SearchDir			Indicates whether the search starts at the beginning or at the end (defaults to ESearchDir::FromStart)
	 * @return					Returns whether the string contains the substring. If the substring is empty, returns true.
	 **/
	[[nodiscard]] UE_FORCEINLINE_HINT bool Contains(const UE_STRING_CLASS& SubStr, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase,
							  ESearchDir::Type SearchDir = ESearchDir::FromStart) const
	{
		return Find(*SubStr, SearchCase, SearchDir) != INDEX_NONE;
	}

	/**
	 * Returns whether this string contains the specified substring.
	 *
	 * @param SubStr			Text to search for
	 * @param SubStrLen			Length of the Text
	 * @param SearchCase		Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @param SearchDir			Indicates whether the search starts at the beginning or at the end (defaults to ESearchDir::FromStart)
	 * @return					Returns whether the string contains the substring. If the substring is empty, returns true.
	 **/
	[[nodiscard]] UE_FORCEINLINE_HINT bool Contains(const ElementType* SubStr, int32 SubStrLen,
		ESearchCase::Type SearchCase = ESearchCase::IgnoreCase, ESearchDir::Type SearchDir = ESearchDir::FromStart) const
	{
		return Find(SubStr, SubStrLen, SearchCase, SearchDir) != INDEX_NONE;
	}

	/**
	 * Searches the string for a character
	 *
	 * @param InChar the character to search for
	 * @param OutIndex out the position the character was found at, INDEX_NONE if return is false
	 * @return true if character was found in this string, otherwise false
	 */
	UE_FORCEINLINE_HINT bool FindChar(ElementType InChar, int32& OutIndex) const
	{
		return Data.Find(InChar, OutIndex);
	}

	/**
	 * Searches the string for the last occurrence of a character
	 *
	 * @param InChar the character to search for
	 * @param OutIndex out the position the character was found at, INDEX_NONE if return is false
	 * @return true if character was found in this string, otherwise false
	 */
	UE_FORCEINLINE_HINT bool FindLastChar(ElementType InChar, int32& OutIndex) const
	{
		return Data.FindLast(InChar, OutIndex);
	}

	/**
	 * Searches an initial substring for the last occurrence of a character which matches the specified predicate.
	 *
	 * @param Pred Predicate that takes a character and returns true if it matches search criteria, false otherwise.
	 * @param Count The number of characters from the front of the string through which to search.
	 *
	 * @return Index of the found character, INDEX_NONE otherwise.
	 */
	template <typename Predicate>
	inline int32 FindLastCharByPredicate(Predicate Pred, int32 Count) const
	{
		check(Count >= 0 && Count <= this->Len());
		return Data.FindLastByPredicate(Pred, Count);
	}

	/**
	 * Searches the string for the last occurrence of a character which matches the specified predicate.
	 *
	 * @param Pred Predicate that takes a character and returns true if it matches search criteria, false otherwise.
	 *
	 * @return Index of the found character, INDEX_NONE otherwise.
	 */
	template <typename Predicate>
	UE_FORCEINLINE_HINT int32 FindLastCharByPredicate(Predicate Pred) const
	{
		return Data.FindLastByPredicate(Pred, this->Len());
	}

	/**
	 * Lexicographically tests whether this string is equivalent to the Other given string
	 * 
	 * @param Other 	The string test against
	 * @param SearchCase 	Whether or not the comparison should ignore case
	 * @return true if this string is lexicographically equivalent to the other, otherwise false
	 */
	[[nodiscard]] inline bool Equals(const UE_STRING_CLASS& Other, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive) const
	{
		int32 Num = Data.Num();
		int32 OtherNum = Other.Data.Num();

		if (Num != OtherNum)
		{
			// Handle special case where FString() == FString("")
			return Num + OtherNum == 1;
		}
		else if (Num > 1)
		{
			if (SearchCase == ESearchCase::CaseSensitive)
			{
				return TCString<ElementType>::Strcmp(Data.GetData(), Other.Data.GetData()) == 0; 
			}
			else
			{
				return TCString<ElementType>::Stricmp(Data.GetData(), Other.Data.GetData()) == 0;
			}
		}

		return true;
	}

	/**
	 * Lexicographically tests how this string compares to the Other given string
	 * 
	 * @param Other 	The string test against
	 * @param SearchCase 	Whether or not the comparison should ignore case
	 * @return 0 if equal, negative if less than, positive if greater than
	 */
	[[nodiscard]] inline int32 Compare(const UE_STRING_CLASS& Other, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive) const
	{
		if (SearchCase == ESearchCase::CaseSensitive)
		{
			return TCString<ElementType>::Strcmp(**this, *Other); 
		}
		else
		{
			return TCString<ElementType>::Stricmp(**this, *Other);
		}
	}

	/**
	 * Splits this string at given string position case sensitive.
	 *
	 * @param InStr The string to search and split at
	 * @param LeftS out the string to the left of InStr, not updated if return is false. LeftS must not point to the same location as RightS, but can point to this.
	 * @param RightS out the string to the right of InStr, not updated if return is false. RightS must not point to the same location as LeftS, but can point to this.
	 * @param SearchCase		Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @param SearchDir			Indicates whether the search starts at the beginning or at the end (defaults to ESearchDir::FromStart)
	 * @return true if string is split, otherwise false
	 */
	CORE_API bool Split(const UE_STRING_CLASS& InStr, UE_STRING_CLASS* LeftS, UE_STRING_CLASS* RightS, ESearchCase::Type SearchCase,
		ESearchDir::Type SearchDir = ESearchDir::FromStart) const;

	/** Split with ESearchCase::IgnoreCase and ESearchDir::FromStart. Allows compiler to avoid branches w/o inlining code. */
	CORE_API bool Split(const UE_STRING_CLASS& InStr, UE_STRING_CLASS* LeftS, UE_STRING_CLASS* RightS) const;

	/** Returns a new string with the characters of this converted to uppercase */
	[[nodiscard]] CORE_API UE_STRING_CLASS ToUpper() const &;

	/**
	 * Converts all characters in this rvalue string to uppercase and moves it into the returned string.
	 * @return a new string with the characters of this converted to uppercase
	 */
	[[nodiscard]] CORE_API UE_STRING_CLASS ToUpper() &&;

	/** Converts all characters in this string to uppercase */
	CORE_API void ToUpperInline();

	/** Returns a new string with the characters of this converted to lowercase */
	[[nodiscard]] CORE_API UE_STRING_CLASS ToLower() const &;

	/**
	 * Converts all characters in this rvalue string to lowercase and moves it into the returned string.
	 * @return a new string with the characters of this converted to lowercase
	 */
	[[nodiscard]] CORE_API UE_STRING_CLASS ToLower() &&;

	/** Converts all characters in this string to lowercase */
	CORE_API void ToLowerInline();

	/** Pad the left of this string for ChCount characters */
	[[nodiscard]] CORE_API UE_STRING_CLASS LeftPad(int32 ChCount) const;

	/** Pad the right of this string for ChCount characters */
	[[nodiscard]] CORE_API UE_STRING_CLASS RightPad(int32 ChCount) const;
	
	/** Returns true if the string only contains numeric characters */
	[[nodiscard]] CORE_API bool IsNumeric() const;
	
	/** Removes spaces from the string.  I.E. "Spaces Are Cool" --> "SpacesAreCool". */
	CORE_API void RemoveSpacesInline();

	/**
	 * Constructs a string similarly to how classic sprintf works.
	 *
	 * @param Fmt  Format string that specifies how the string should be built optionally using additional args. Refer to standard printf format.
	 * @param ...  Depending on format function may require additional arguments to build output object.
	 *
	 * @returns A string that was constructed using format and additional parameters.
	 */
	template <typename... Types>
	[[nodiscard]] static UE_STRING_CLASS Printf(UE::Core::TCheckedFormatString<FmtCharType, Types...> Fmt, Types... Args)
	{
		return PrintfImpl((const ElementType*)Fmt.FormatString, Args...);
	}
#ifdef UE_STRING_PRINTF_FMT_CHARTYPE2
	template <typename... Types>
	[[nodiscard]] static UE_STRING_CLASS Printf(UE::Core::TCheckedFormatString<FmtCharType2, Types...> Fmt, Types... Args)
	{
		return PrintfImpl((const ElementType*)Fmt.FormatString, Args...);
	}
#endif

	/**
	 * Just like Printf, but appends the formatted text to the existing string instead.
	 * @return a reference to the modified string, so that it can be chained
	 */
	template <typename... Types>
	UE_STRING_CLASS& Appendf(UE::Core::TCheckedFormatString<FmtCharType, Types...> Fmt, Types... Args)
	{
		AppendfImpl(*this, (const ElementType*)Fmt.FormatString, Args...);
		return *this;
	}
#ifdef UE_STRING_PRINTF_FMT_CHARTYPE2
	template <typename... Types>
	UE_STRING_CLASS& Appendf(UE::Core::TCheckedFormatString<FmtCharType2, Types...> Fmt, Types... Args)
	{
		AppendfImpl(*this, (const ElementType*)Fmt.FormatString, Args...);
		return *this;
	}
#endif

private:
	static CORE_API UE_STRING_CLASS VARARGS PrintfImpl(const ElementType* Fmt, ...);
	static CORE_API void VARARGS AppendfImpl(UE_STRING_CLASS& AppendToMe, const ElementType* Fmt, ...);
public:

	/**
	 * Format the specified string using the specified arguments. Replaces instances of { Argument } with keys in the map matching 'Argument'
	 * @param InFormatString		A string representing the format expression
	 * @param InNamedArguments		A map of named arguments that match the tokens specified in InExpression
	 * @return A string containing the formatted text
	 */
	[[nodiscard]] static UE_STRING_CLASS Format(const FmtCharType* InFormatString, const UE_JOIN(UE_STRING_CLASS, FormatNamedArguments)& InNamedArguments)
	{
		return FormatImpl((const ElementType*)InFormatString, InNamedArguments);
	}
#ifdef UE_STRING_PRINTF_FMT_CHARTYPE2
	[[nodiscard]] static UE_STRING_CLASS Format(const FmtCharType2* InFormatString, const UE_JOIN(UE_STRING_CLASS, FormatNamedArguments)& InNamedArguments)
	{
		return FormatImpl((const ElementType*)InFormatString, InNamedArguments);
	}
#endif

	/**
	 * Format the specified string using the specified arguments. Replaces instances of {0} with indices from the given array matching the index specified in the token
	 * @param InFormatString		A string representing the format expression
	 * @param InOrderedArguments	An array of ordered arguments that match the tokens specified in InExpression
	 * @return A string containing the formatted text
	 */
	[[nodiscard]] static UE_STRING_CLASS Format(const FmtCharType* InFormatString, const FStringFormatOrderedArguments& InOrderedArguments)
	{
		return FormatImpl((const ElementType*)InFormatString, InOrderedArguments);
	}
#ifdef UE_STRING_PRINTF_FMT_CHARTYPE2
	[[nodiscard]] static UE_STRING_CLASS Format(const FmtCharType2* InFormatString, const FStringFormatOrderedArguments& InOrderedArguments)
	{
		return FormatImpl((const ElementType*)InFormatString, InOrderedArguments);
	}
#endif

private:
	static CORE_API UE_STRING_CLASS FormatImpl(const ElementType* InFormatString, const UE_JOIN(UE_STRING_CLASS, FormatNamedArguments)& InNamedArguments);
	static CORE_API UE_STRING_CLASS FormatImpl(const ElementType* InFormatString, const FStringFormatOrderedArguments& InOrderedArguments);
public:

	/** Returns a string containing only the Ch character */
	[[nodiscard]] static CORE_API UE_STRING_CLASS Chr(ElementType Ch);

	/**
	 * Returns a string that is full of a variable number of characters
	 *
	 * @param NumCharacters Number of characters to put into the string
	 * @param Char Character to put into the string
	 * 
	 * @return The string of NumCharacters characters.
	 */
	[[nodiscard]] static CORE_API UE_STRING_CLASS ChrN(int32 NumCharacters, ElementType Char);

	/**
	 * Serializes the string.
	 *
	 * @param Ar Reference to the serialization archive.
	 * @param Str Reference to the string being serialized.
	 *
	 * @return Reference to the Archive after serialization.
	 */
	friend CORE_API FArchive& operator<<(FArchive& Ar, UE_STRING_CLASS& Str);


	/**
	 * Test whether this string starts with given prefix.
	 *
	 * @param InPrefix    The prefix to search for at the start of the string.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if this string begins with specified text, false otherwise
	 */
	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(
			TIsContiguousContainer<CharRangeType>::Value &&
			!std::is_array_v<std::remove_reference_t<CharRangeType>>&&
			TIsCharType_V<CharRangeElementType> &&
			!std::is_base_of_v<UE_STRING_CLASS, std::decay_t<CharRangeType>>
		)
	>
	[[nodiscard]] bool StartsWith(CharRangeType&& InPrefix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const
	{
		static_assert(std::is_same_v<CharRangeElementType, ElementType>, "Expected a range of characters");
		return StartsWith(GetData(InPrefix), GetNum(InPrefix), SearchCase);
	}

	/**
	 * Test whether this string starts with given prefix.
	 *
	 * @param InPrefix    The prefix to search for at the start of the string.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if this string begins with specified text, false otherwise
	 */
	[[nodiscard]] bool StartsWith(const ElementType* InPrefix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const
	{
		return StartsWith(InPrefix, InPrefix ? TCString<ElementType>::Strlen(InPrefix) : 0, SearchCase);
	}

	/**
	 * Test whether this string starts with given prefix.
	 *
	 * @param InPrefix    The prefix to search for at the start of the string.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if this string begins with specified text, false otherwise
	 */
	[[nodiscard]] bool StartsWith(const UE_STRING_CLASS& InPrefix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const
	{
		return StartsWith(*InPrefix, InPrefix.Len(), SearchCase);
	}

	/**
	 * Test whether this string starts with given prefix.
	 *
	 * @param InPrefix    The prefix to search for at the start of the string.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if this string begins with specified text, false otherwise
	 */
	[[nodiscard]] CORE_API bool StartsWith(const ElementType* InPrefix, int32 InPrefixLen, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const;

	/**
	 * Test whether this string ends with given suffix.
	 *
	 * @param InSuffix    The suffix to search for at the end of the string.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if this string ends with specified text, false otherwise
	 */
	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(
			TIsContiguousContainer<CharRangeType>::Value &&
			!std::is_array_v<std::remove_reference_t<CharRangeType>>&&
			TIsCharType_V<CharRangeElementType> &&
			!std::is_base_of_v<UE_STRING_CLASS, std::decay_t<CharRangeType>>
		)
	>
	[[nodiscard]] bool EndsWith(CharRangeType&& InSuffix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const
	{
		static_assert(std::is_same_v<CharRangeElementType, ElementType>, "Expected a range of characters");
		return EndsWith(GetData(InSuffix), GetNum(InSuffix), SearchCase);
	}

	/**
	 * Test whether this string ends with given suffix.
	 *
	 * @param InSuffix    The suffix to search for at the end of the string.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if this string ends with specified text, false otherwise
	 */
	[[nodiscard]] bool EndsWith(const ElementType* InSuffix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const
	{
		return EndsWith(InSuffix, InSuffix ? TCString<ElementType>::Strlen(InSuffix) : 0, SearchCase);
	}

	/**
	 * Test whether this string ends with given suffix.
	 *
	 * @param InSuffix    The suffix to search for at the end of the string.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if this string ends with specified text, false otherwise
	 */
	[[nodiscard]] bool EndsWith(const UE_STRING_CLASS& InSuffix, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const
	{
		return EndsWith(*InSuffix, InSuffix.Len(), SearchCase);
	}

	/**
	 * Test whether this string ends with given suffix.
	 *
	 * @param InSuffix    The suffix to search for at the end of the string.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if this string ends with specified text, false otherwise
	 */
	[[nodiscard]] CORE_API bool EndsWith(const ElementType* InSuffix, int32 InSuffixLen, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const;

	/**
	 * Searches this string for a given wild card
	 *
	 * @param Wildcard		*?-type wildcard. '*' matches 0 or more arbitrary characters, '?' matches either an arbitrary character or nothing, all other characters match themselves.
	 * @param SearchCase	Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if this string matches the *?-type wildcard given. 
	 * @warning This is a simple, SLOW routine. Use with caution. Also note that '?' can match nothing (not common for wildcard patterns).
	 */
	template <
		typename CharRangeType,
		typename CharRangeElementType = TElementType_T<CharRangeType>
		UE_REQUIRES(
			TIsContiguousContainer<CharRangeType>::Value &&
			!std::is_array_v<std::remove_reference_t<CharRangeType>>&&
			TIsCharType_V<CharRangeElementType> &&
			!std::is_base_of_v<UE_STRING_CLASS, std::decay_t<CharRangeType>>
		)
	>
	[[nodiscard]] bool MatchesWildcard(CharRangeType&& Wildcard, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const
	{
		static_assert(std::is_same_v<CharRangeElementType, ElementType>, "Expected a range of characters");
		return MatchesWildcard(GetData(Wildcard), GetNum(Wildcard), SearchCase);
	}

	/**
	 * Searches this string for a given wild card
	 *
	 * @param Wildcard		*?-type wildcard. '*' matches 0 or more arbitrary characters, '?' matches either an arbitrary character or nothing, all other characters match themselves.
	 * @param SearchCase	Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if this string matches the *?-type wildcard given.
	 * @warning This is a simple, SLOW routine. Use with caution. Also note that '?' can match nothing (not common for wildcard patterns).
	 */
	[[nodiscard]] bool MatchesWildcard(const ElementType* Wildcard, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const
	{
		return MatchesWildcard(Wildcard, Wildcard ? TCString<ElementType>::Strlen(Wildcard) : 0, SearchCase);
	}

	/**
	 * Searches this string for a given wild card
	 *
	 * @param Wildcard		*?-type wildcard. '*' matches 0 or more arbitrary characters, '?' matches either an arbitrary character or nothing, all other characters match themselves.
	 * @param SearchCase	Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if this string matches the *?-type wildcard given.
	 * @warning This is a simple, SLOW routine. Use with caution. Also note that '?' can match nothing (not common for wildcard patterns).
	 */
	[[nodiscard]] bool MatchesWildcard(const UE_STRING_CLASS& Wildcard, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const
	{
		return MatchesWildcard(*Wildcard, Wildcard.Len(), SearchCase);
	}

	/**
	 * Searches this string for a given wild card
	 *
	 * @param Wildcard		*?-type wildcard. '*' matches 0 or more arbitrary characters, '?' matches either an arbitrary character or nothing, all other characters match themselves.
	 * @param WildcardLen  The length of Wildcard
	 * @param SearchCase   Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return true if this string matches the *?-type wildcard given.
	 * @warning This is a simple, SLOW routine. Use with caution. Also note that '?' can match nothing (not common for wildcard patterns).
	 */
	[[nodiscard]] CORE_API bool MatchesWildcard(const ElementType* Wildcard, int32 WildcardLen, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const;

	/**
	 * Removes whitespace characters from the start and end of this string. Modifies the string in-place.
	 */
	CORE_API void TrimStartAndEndInline();

	/**
	 * Removes whitespace characters from the start and end of this string.
	 * @note Unlike Trim() this function returns a copy, and does not mutate the string.
	 */
	[[nodiscard]] CORE_API UE_STRING_CLASS TrimStartAndEnd() const &;

	/**
	 * Removes whitespace characters from the start and end of this string.
	 * @note Unlike Trim() this function returns a copy, and does not mutate the string.
	 */
	[[nodiscard]] CORE_API UE_STRING_CLASS TrimStartAndEnd() &&;

	/**
	 * Removes whitespace characters from the start of this string. Modifies the string in-place.
	 */
	CORE_API void TrimStartInline();

	/**
	 * Removes whitespace characters from the start of this string.
	 * @note Unlike Trim() this function returns a copy, and does not mutate the string.
	 */
	[[nodiscard]] CORE_API UE_STRING_CLASS TrimStart() const &;

	/**
	 * Removes whitespace characters from the start of this string.
	 * @note Unlike Trim() this function returns a copy, and does not mutate the string.
	 */
	[[nodiscard]] CORE_API UE_STRING_CLASS TrimStart() &&;

	/**
	 * Removes whitespace characters from the end of this string. Modifies the string in-place.
	 */
	CORE_API void TrimEndInline();

	/**
	 * Removes whitespace characters from the end of this string.
	 * @note Unlike TrimTrailing() this function returns a copy, and does not mutate the string.
	 */
	[[nodiscard]] CORE_API UE_STRING_CLASS TrimEnd() const &;

	/**
	 * Removes whitespace characters from the end of this string.
	 * @note Unlike TrimTrailing() this function returns a copy, and does not mutate the string.
	 */
	[[nodiscard]] CORE_API UE_STRING_CLASS TrimEnd() &&;

	/** 
	 * Trims the inner array after the null terminator.
	 */
	CORE_API void TrimToNullTerminator();


	/**
	 * Trims wrapping quotation marks from this string.
	 */
	CORE_API void TrimQuotesInline(bool* bQuotesRemoved = nullptr);

	/**
	* Trims a single character from the start and end of the string (removes at max one instance in the beginning and end of the string).
	* @see TrimChar for a variant that returns a modified copy of the string
	*/
	CORE_API void TrimCharInline(ElementType CharacterToTrim, bool* bCharRemoved);

	/**
	 * Returns a copy of this string with wrapping quotation marks removed.
	 */
	[[nodiscard]] CORE_API UE_STRING_CLASS TrimQuotes(bool* bQuotesRemoved = nullptr) const &;

	/**
	 * Returns this string with wrapping quotation marks removed.
	 */
	[[nodiscard]] CORE_API UE_STRING_CLASS TrimQuotes(bool* bQuotesRemoved = nullptr) &&;
	
	/**
	* Returns a copy of this string with wrapping CharacterToTrim removed (removes at max one instance in the beginning and end of the string).
	* @see TrimCharInline for an inline variant
	*/
	[[nodiscard]] CORE_API UE_STRING_CLASS TrimChar(ElementType CharacterToTrim, bool* bCharRemoved = nullptr) const &;

	/**
	* Returns a copy of this string with wrapping CharacterToTrim removed (removes at max one instance in the beginning and end of the string).
	*/
	[[nodiscard]] CORE_API UE_STRING_CLASS TrimChar(ElementType CharacterToTrim, bool* bCharRemoved = nullptr) &&;

	/**
	 * Breaks up a delimited string into elements of a string array.
	 *
	 * @param OutArray      The array to fill with the string pieces
	 * @param pchDelim      The string to delimit on
	 * @param bInCullEmpty  If true, empty strings are not added to the array
	 *
	 * @return	The number of elements in OutArray
	 */
	CORE_API int32 ParseIntoArray(TArray<UE_STRING_CLASS>& OutArray, const ElementType* pchDelim, bool bInCullEmpty = true) const;

	/**
	 * Breaks up a delimited string into elements of a string array, using any whitespace and an 
	 * optional extra delimter, like a ","
	 * @warning Caution!! this routine is O(N^2) allocations...use it for parsing very short text or not at all! 
	 *
	 * @param OutArray       The array to fill with the string pieces
	 * @param pchExtraDelim  The string to delimit on
	 * @param bInCullEmpty   If true, empty strings are not added to the array
	 *
	 * @return	The number of elements in OutArray
	 */
	CORE_API int32 ParseIntoArrayWS(TArray<UE_STRING_CLASS>& OutArray, const ElementType* pchExtraDelim = nullptr, bool bInCullEmpty = true) const;

	/**
	* Breaks up a delimited string into elements of a string array, using line ending characters
	* @warning Caution!! this routine is O(N^2) allocations...use it for parsing very short text or not at all!
	*
	* @param OutArray      The array to fill with the string pieces
	* @param bInCullEmpty  If true, empty strings are not added to the array
	*
	* @return	The number of elements in OutArray
	*/
	CORE_API int32 ParseIntoArrayLines(TArray<UE_STRING_CLASS>& OutArray, bool bInCullEmpty = true) const;

	/**
	* Breaks up a delimited string into elements of a string array, using the given delimiters
	* @warning Caution!! this routine is O(N^2) allocations...use it for parsing very short text or not at all!
	*
	* @param OutArray      The array to fill with the string pieces
	* @param DelimArray    The strings to delimit on
	* @param NumDelims     The number of delimiters.
	* @param bInCullEmpty  If true, empty strings are not added to the array
	*
	* @return	The number of elements in OutArray
	*/
	CORE_API int32 ParseIntoArray(TArray<UE_STRING_CLASS>& OutArray, const ElementType*const* DelimArray, int32 NumDelims, bool bInCullEmpty = true) const;

	/**
	 * Takes an array of strings and removes any zero length entries.
	 *
	 * @param	InOutArray  The array to cull
	 *
	 * @return	The number of elements left in InOutArray
	 */
	static CORE_API int32 CullArray(TArray<UE_STRING_CLASS>* InOutArray);

	/**
	 * Returns a copy of this string, with the characters in reverse order
	 */
	[[nodiscard]] CORE_API UE_STRING_CLASS Reverse() const &;

	/**
	 * Returns this string, with the characters in reverse order
	 */
	[[nodiscard]] CORE_API UE_STRING_CLASS Reverse() &&;

	/**
	 * Reverses the order of characters in this string
	 */
	CORE_API void ReverseString();

	/**
	 * Returns a copy of the string with all occurrences of a substring replaced.
	 *
	 * @param From        The substring to replace.
	 * @param To          The substring to insert in its place.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return a copy of this string with the replacement made
	 */
	[[nodiscard]] CORE_API UE_STRING_CLASS Replace(const ElementType* From, const ElementType* To, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const &;

	/**
	 * Returns a copy of the string with all occurrences of a substring replaced.
	 *
	 * @param From        The substring to replace.
	 * @param To          The substring to insert in its place.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @return a copy of this string with the replacement made
	 */
	[[nodiscard]] CORE_API UE_STRING_CLASS Replace(const ElementType* From, const ElementType* To, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) &&;

	/**
	 * Replaces all occurrences of a substring in this string.
	 *
	 * @param From        The substring to replace.
	 * @param To          The substring to insert in its place.
	 * @param SearchCase  Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 *
	 * @return	the number of occurrences of SearchText that were replaced.
	 */
	CORE_API int32 ReplaceInline(const ElementType* SearchText, const ElementType* ReplacementText, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase);

	/**
	 * Replaces all occurrences of a character in this string.
	 *
	 * @param SearchChar      Character to remove from this string
	 * @param ReplacementChar Replacement character
	 * @param SearchCase      Indicates whether the search is case sensitive or not (defaults to ESearchCase::IgnoreCase)
	 * @note no dynamic allocation
	 */
	void ReplaceCharInline(ElementType From, ElementType To, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
	{
		if (SearchCase == ESearchCase::IgnoreCase && TChar<ElementType>::IsAlpha(From))
		{
			ReplaceCharInlineIgnoreCase(From, To);
		}
		else
		{
			ReplaceCharInlineCaseSensitive(From, To);
		}
	}

private:
	CORE_API void ReplaceCharInlineCaseSensitive(ElementType From, ElementType To);
	CORE_API void ReplaceCharInlineIgnoreCase(ElementType From, ElementType To);

public:

	/**
	 * Returns a copy of this string with all quote marks escaped (unless the quote is already escaped)
	 */
	[[nodiscard]] UE_STRING_CLASS ReplaceQuotesWithEscapedQuotes() const &
	{
		UE_STRING_CLASS Result(*this);
		return MoveTemp(Result).ReplaceQuotesWithEscapedQuotes();
	}

	/**
	 * Returns a copy of this string with all quote marks escaped (unless the quote is already escaped)
	 */
	[[nodiscard]] CORE_API UE_STRING_CLASS ReplaceQuotesWithEscapedQuotes() &&;

	/**
	 * Replaces certain characters with the "escaped" version of that character (i.e. replaces "\n" with "\\n").
	 * The characters supported are: { \n, \r, \t, \', \", \\ }.
	 *
	 * @param	Chars	by default, replaces all supported characters; this parameter allows you to limit the replacement to a subset.
	 */
	CORE_API void ReplaceCharWithEscapedCharInline(const TArray<ElementType>* Chars = nullptr);

	/**
	 * Replaces certain characters with the "escaped" version of that character (i.e. replaces "\n" with "\\n").
	 * The characters supported are: { \n, \r, \t, \', \", \\ }.
	 *
	 * @param	Chars	by default, replaces all supported characters; this parameter allows you to limit the replacement to a subset.
	 *
	 * @return	a string with all control characters replaced by the escaped version.
	 */
	[[nodiscard]] UE_STRING_CLASS ReplaceCharWithEscapedChar(const TArray<ElementType>* Chars = nullptr) const &
	{
		UE_STRING_CLASS Result(*this);
		Result.ReplaceCharWithEscapedCharInline(Chars);
		return Result;
	}

	/**
	 * Replaces certain characters with the "escaped" version of that character (i.e. replaces "\n" with "\\n").
	 * The characters supported are: { \n, \r, \t, \', \", \\ }.
	 *
	 * @param	Chars	by default, replaces all supported characters; this parameter allows you to limit the replacement to a subset.
	 *
	 * @return	a string with all control characters replaced by the escaped version.
	 */
	[[nodiscard]] UE_STRING_CLASS ReplaceCharWithEscapedChar(const TArray<ElementType>* Chars = nullptr) &&
	{
		ReplaceCharWithEscapedCharInline(Chars);
		return MoveTemp(*this);
	}

	/**
	 * Removes the escape backslash for all supported characters, replacing the escape and character with the non-escaped version.  (i.e.
	 * replaces "\\n" with "\n".  Counterpart to ReplaceCharWithEscapedCharInline().
	 */
	CORE_API void ReplaceEscapedCharWithCharInline(const TArray<ElementType>* Chars = nullptr);

	/**
	 * Removes the escape backslash for all supported characters, replacing the escape and character with the non-escaped version.  (i.e.
	 * replaces "\\n" with "\n".  Counterpart to ReplaceCharWithEscapedChar().
	 * @return copy of this string with replacement made
	 */
	[[nodiscard]] UE_STRING_CLASS ReplaceEscapedCharWithChar(const TArray<ElementType>* Chars = nullptr) const &
	{
		UE_STRING_CLASS Result(*this);
		Result.ReplaceEscapedCharWithCharInline(Chars);
		return Result;
	}

	/**
	 * Removes the escape backslash for all supported characters, replacing the escape and character with the non-escaped version.  (i.e.
	 * replaces "\\n" with "\n".  Counterpart to ReplaceCharWithEscapedChar().
	 * @return copy of this string with replacement made
	 */
	[[nodiscard]] UE_STRING_CLASS ReplaceEscapedCharWithChar(const TArray<ElementType>* Chars = nullptr) &&
	{
		ReplaceEscapedCharWithCharInline(Chars);
		return MoveTemp(*this);
	}

	/**
	 * Replaces all instances of '\t' with TabWidth number of spaces
	 * @param InSpacesPerTab - Number of spaces that a tab represents
	 */
	CORE_API void ConvertTabsToSpacesInline(const int32 InSpacesPerTab);

	/** 
	 * Replaces all instances of '\t' with TabWidth number of spaces
	 * @param InSpacesPerTab - Number of spaces that a tab represents
	 * @return copy of this string with replacement made
	 */
	[[nodiscard]] UE_STRING_CLASS ConvertTabsToSpaces(const int32 InSpacesPerTab) const &
	{
		UE_STRING_CLASS FinalString(*this);
		FinalString.ConvertTabsToSpacesInline(InSpacesPerTab);
		return FinalString;
	}

	/**
	 * Replaces all instances of '\t' with TabWidth number of spaces
	 * @param InSpacesPerTab - Number of spaces that a tab represents
	 * @return copy of this string with replacement made
	 */
	[[nodiscard]] UE_STRING_CLASS ConvertTabsToSpaces(const int32 InSpacesPerTab) &&
	{
		ConvertTabsToSpacesInline(InSpacesPerTab);
		return MoveTemp(*this);
	}

	// Takes the number passed in and formats the string in comma format (12345 becomes "12,345")
	[[nodiscard]] static CORE_API UE_STRING_CLASS FormatAsNumber(int32 InNumber);

	// To allow more efficient memory handling, automatically adds one for the string termination.
	CORE_API void Reserve(int32 CharacterCount);

	/**
	 * Serializes a string as ANSI char array.
	 *
	 * @param	String			String to serialize
	 * @param	Ar				Archive to serialize with
	 * @param	MinCharacters	Minimum number of characters to serialize.
	 */
	CORE_API void SerializeAsANSICharArray(FArchive& Ar, int32 MinCharacters = 0) const;


	/** Converts an integer to a string. */
	[[nodiscard]] static inline UE_STRING_CLASS FromInt(int32 Num)
	{
		UE_STRING_CLASS Ret;
		Ret.AppendInt(Num); 
		return Ret;
	}

	/** appends the integer InNum to this string */
	CORE_API void AppendInt(int32 InNum);

	/**
	 * Converts a string into a boolean value
	 *   1, "True", "Yes", FCoreTexts::True, FCoreTexts::Yes, and non-zero integers become true
	 *   0, "False", "No", FCoreTexts::False, FCoreTexts::No, and unparsable values become false
	 *
	 * @return The boolean value
	 */
	[[nodiscard]] CORE_API bool ToBool() const;

	/**
	 * Converts a buffer to a string
	 *
	 * @param SrcBuffer the buffer to stringify
	 * @param SrcSize the number of bytes to convert
	 *
	 * @return the blob in string form
	 */
	[[nodiscard]] static CORE_API UE_STRING_CLASS FromBlob(const uint8* SrcBuffer, const uint32 SrcSize);

	/**
	 * Converts a string into a buffer
	 *
	 * @param DestBuffer the buffer to fill with the string data
	 * @param DestSize the size of the buffer in bytes (must be at least string len / 3)
	 *
	 * @return true if the conversion happened, false otherwise
	 */
	static CORE_API bool ToBlob(const UE_STRING_CLASS& Source,uint8* DestBuffer, const uint32 DestSize);

	/**
	 * Converts a buffer to a string by hex-ifying the elements
	 *
	 * @param SrcBuffer the buffer to stringify
	 * @param SrcSize the number of bytes to convert
	 *
	 * @return the blob in string form
	 */
	[[nodiscard]] static CORE_API UE_STRING_CLASS FromHexBlob(const uint8* SrcBuffer, const uint32 SrcSize);

	/**
	 * Converts a string into a buffer
	 *
	 * @param DestBuffer the buffer to fill with the string data
	 * @param DestSize the size of the buffer in bytes (must be at least string len / 2)
	 *
	 * @return true if the conversion happened, false otherwise
	 */
	static CORE_API bool ToHexBlob(const UE_STRING_CLASS& Source, uint8* DestBuffer, const uint32 DestSize);

	/**
	 * Converts a float string with the trailing zeros stripped
	 * For example - 1.234 will be "1.234" rather than "1.234000"
	 * 
	 * @param	InFloat					The float to sanitize
	 * @param	InMinFractionalDigits	The minimum number of fractional digits the number should have (will be padded with zero)
	 *
	 * @return sanitized string version of float
	 */
	[[nodiscard]] static CORE_API UE_STRING_CLASS SanitizeFloat(double InFloat, const int32 InMinFractionalDigits = 1);

	/**
	 * Joins a range of 'something that can be concatentated to strings with +=' together into a single string with separators.
	 *
	 * @param	Range		The range of 'things' to concatenate.
	 * @param	Separator	The string used to separate each element.
	 *
	 * @return	The final, joined, separated string.
	 */
	template <typename RangeType>
	[[nodiscard]] static UE_STRING_CLASS Join(const RangeType& Range, const ElementType* Separator)
	{
		UE_STRING_CLASS Result;
		bool            First = true;
		for (const auto& Element : Range)
		{
			if (First)
			{
				First = false;
			}
			else
			{
				Result += Separator;
			}

			Result += Element;
		}

		return Result;
	}

	/**
	 * Joins a range of elements together into a single string with separators using a projection function.
	 *
	 * @param	Range		The range of 'things' to concatenate.
	 * @param	Separator	The string used to separate each element.
	 * @param	Proj		The projection used to get a string for each element.
	 *
	 * @return	The final, joined, separated string.
	 */
	template <typename RangeType, typename ProjectionType>
	[[nodiscard]] static UE_STRING_CLASS JoinBy(const RangeType& Range, const ElementType* Separator, ProjectionType Proj)
	{
		UE_STRING_CLASS Result;
		bool            First = true;
		for (const auto& Element : Range)
		{
			if (First)
			{
				First = false;
			}
			else
			{
				Result += Separator;
			}

			Result += Invoke(Proj, Element);
		}

		return Result;
	}

	UE_FORCEINLINE_HINT void CountBytes(FArchive& Ar) const
	{
		Data.CountBytes(Ar);
	}

	/** Case insensitive string hash function. */
	[[nodiscard]] friend UE_FORCEINLINE_HINT uint32 GetTypeHash(const UE_STRING_CLASS& S)
	{
		// This must match the GetTypeHash behavior of FStringView
		return FCrc::Strihash_DEPRECATED(S.Len(), *S);
	}
};

template<> struct TIsZeroConstructType<UE_STRING_CLASS> { enum { Value = true }; };

template<>
struct TNameOf<UE_STRING_CLASS>
{
	[[nodiscard]] UE_FORCEINLINE_HINT static TCHAR const* GetName()
	{
		return TEXT(UE_STRINGIZE(UE_STRING_CLASS));
	}
};

[[nodiscard]] inline UE_STRING_CLASS::ElementType* GetData(UE_STRING_CLASS& String)
{
	return String.GetCharArray().GetData();
}

[[nodiscard]] inline const UE_STRING_CLASS::ElementType* GetData(const UE_STRING_CLASS& String)
{
	return String.GetCharArray().GetData();
}

[[nodiscard]] inline int32 GetNum(const UE_STRING_CLASS& String)
{
	return String.Len();
}

/** Append bytes as uppercase hex string */
CORE_API void BytesToHex(const uint8* Bytes, int32 NumBytes, UE_STRING_CLASS& Out);
/** Append bytes as lowercase hex string */
CORE_API void BytesToHexLower(const uint8* Bytes, int32 NumBytes, UE_STRING_CLASS& Out);

/** 
 * Convert a string of Hex digits into the byte array.
 * @param HexString		The string of Hex values
 * @param OutBytes		Ptr to memory must be preallocated large enough
 * @return	The number of bytes copied
 */
CORE_API int32 HexToBytes(const UE_STRING_CLASS& HexString, uint8* OutBytes);

/**
 * Generalized API to convert something to a string. Function named after the (deprecated) Lex namespace, which
 * was deprecated because introducing customization points in a nested namespace didn't work in generic code because
 * it foiled 2-phase template instantiating compilers, which would bind to the qualified name (LexToString) in the first phase,
 * preventing future overloads defined in later headers to be considered for binding.
 */
 /**
 *	Expected functions in this namespace are as follows:
 *		bool								LexTryParseString(T& OutValue, const TCHAR* Buffer);
 *		void 								LexFromString(T& OutValue, const TCHAR* Buffer);
 *		<implicitly convertible to string>	LexToString(T);
 *		                    ^-- Generally this means it can return either a string or const TCHAR* 
 *		                        Generic code that uses ToString should assign to a string or forward along to other functions
 *		                        that accept types that are also implicitly convertible to a string 
 *
 *	Implement custom functionality externally.
 */

 /** Convert a string buffer to intrinsic types */
inline void LexFromString(int8& OutValue, 				const UE_STRING_CHARTYPE* Buffer)	{	OutValue = (int8)TCString<UE_STRING_CHARTYPE>::Atoi(Buffer);		}
inline void LexFromString(int16& OutValue,				const UE_STRING_CHARTYPE* Buffer)	{	OutValue = (int16)TCString<UE_STRING_CHARTYPE>::Atoi(Buffer);		}
inline void LexFromString(int32& OutValue,				const UE_STRING_CHARTYPE* Buffer)	{	OutValue = (int32)TCString<UE_STRING_CHARTYPE>::Atoi(Buffer);		}
inline void LexFromString(int64& OutValue,				const UE_STRING_CHARTYPE* Buffer)	{	OutValue = TCString<UE_STRING_CHARTYPE>::Atoi64(Buffer);	}
inline void LexFromString(uint8& OutValue,				const UE_STRING_CHARTYPE* Buffer)	{	OutValue = (uint8)TCString<UE_STRING_CHARTYPE>::Atoi(Buffer);		}
inline void LexFromString(uint16& OutValue, 			const UE_STRING_CHARTYPE* Buffer)	{	OutValue = (uint16)TCString<UE_STRING_CHARTYPE>::Atoi(Buffer);		}
inline void LexFromString(uint32& OutValue, 			const UE_STRING_CHARTYPE* Buffer)	{	OutValue = (uint32)TCString<UE_STRING_CHARTYPE>::Atoi64(Buffer);	}	//64 because this unsigned and so Atoi might overflow
inline void LexFromString(uint64& OutValue, 			const UE_STRING_CHARTYPE* Buffer)	{	OutValue = TCString<UE_STRING_CHARTYPE>::Strtoui64(Buffer, nullptr, 0); }
inline void LexFromString(float& OutValue,				const UE_STRING_CHARTYPE* Buffer)	{	OutValue = TCString<UE_STRING_CHARTYPE>::Atof(Buffer);		}
inline void LexFromString(double& OutValue, 			const UE_STRING_CHARTYPE* Buffer)	{	OutValue = TCString<UE_STRING_CHARTYPE>::Atod(Buffer);		}
inline void LexFromString(bool& OutValue, 				const UE_STRING_CHARTYPE* Buffer)	{	OutValue = TCString<UE_STRING_CHARTYPE>::ToBool(Buffer);	}
inline void LexFromString(UE_STRING_CLASS& OutValue,	const UE_STRING_CHARTYPE* Buffer)	{	OutValue = Buffer;						}

/** Parse a string into this type, returning whether it was successful */
/** Specialization for arithmetic types */
template <UE::CArithmetic T>
bool LexTryParseString(T& OutValue, const UE_STRING_CHARTYPE* Buffer)
{
	if (Buffer[0] == '\0')
	{
		OutValue = 0;
		return false;
	}

	LexFromString(OutValue, Buffer);
	if (OutValue == 0 && FMath::IsFinite((float)OutValue)) //@TODO:FLOATPRECISION: ? huh ?
	{
		bool bSawZero = false;
		UE_STRING_CHARTYPE C = *Buffer;
		while (C != CHARTEXT(UE_STRING_CHARTYPE, '\0') && (C == CHARTEXT(UE_STRING_CHARTYPE, '+') || C == CHARTEXT(UE_STRING_CHARTYPE, '-') || TChar<UE_STRING_CHARTYPE>::IsWhitespace(C)))
		{
			C = *(++Buffer);
		}

		while (C != CHARTEXT(UE_STRING_CHARTYPE, '\0') && !TChar<UE_STRING_CHARTYPE>::IsWhitespace(C) && (std::is_floating_point_v<T> || C != CHARTEXT(UE_STRING_CHARTYPE, '.')))
		{
			bSawZero = bSawZero || (C == CHARTEXT(UE_STRING_CHARTYPE, '0'));
			if (!bSawZero && C != CHARTEXT(UE_STRING_CHARTYPE, '.'))
			{
				return false;
			}

			C = *(++Buffer);
		}
		return bSawZero;
	}

	return true;
}

/** Try and parse a bool - always returns true */
inline bool LexTryParseString(bool& OutValue, const UE_STRING_CHARTYPE* Buffer)
{
	LexFromString(OutValue, Buffer);
	return true;
}
template <typename StringType = FString>
[[nodiscard]] UE_FORCEINLINE_HINT StringType LexToString(UE_STRING_CLASS&& Str)
{
	return MoveTemp(Str);
}

template <typename StringType = FString>
[[nodiscard]] UE_FORCEINLINE_HINT StringType LexToString(const UE_STRING_CLASS& Str)
{
	return Str;
}

/**
 * Gets a non-owning character pointer from a string type.
 *
 * Can be used generically to get a const char pointer, when it is not known if the argument is a char pointer or a string:
 *
 * template <typename T>
 * void LogValue(const T& Val)
 * {
 *     Logf(TEXT("Value: %s"), ToCStr(LexToString(Val)));
 * }
 */
[[nodiscard]] UE_FORCEINLINE_HINT const UE_STRING_CLASS::ElementType* ToCStr(const UE_STRING_CLASS& Str)
{
	return *Str;
}

/**
 * A helper function to find closing parenthesis that matches the first open parenthesis found. The open parenthesis
 * referred to must be at or further up from the start index.
 *
 * @param TargetString      The string to search in
 * @param StartSearch       The index to start searching at
 * @return the index in the given string of the closing parenthesis
 */
[[nodiscard]] CORE_API int32 FindMatchingClosingParenthesis(const UE_STRING_CLASS& TargetString, const int32 StartSearch = 0);

/**
* Given a display label string, generates a slug string that only contains valid characters for an FName.
* For example, "[MyObject]: Object Label" becomes "MyObjectObjectLabel" FName slug.
*
* @param DisplayLabel The label string to convert to an FName
*
* @return	The slugged string
*/
[[nodiscard]] CORE_API UE_STRING_CLASS SlugStringForValidName(const UE_STRING_CLASS& DisplayString, const TCHAR* ReplaceWith = TEXT(""));

#undef UE_INCLUDETOOL_IGNORE_INCONSISTENT_STATE
