// Copyright Epic Games, Inc. All Rights Reserved.

/*******************************************************************************************************
 * NOTICE                                                                                              *
 *                                                                                                     *
 * This file is not intended to be included directly - it is only intended to contain the includes for *
 * UnrealString.h.inl.                                                                                 *
 *******************************************************************************************************/

#ifdef UE_STRING_CLASS
	#error "UnrealStringIncludes.h.inl should not be included after defining UE_STRING_CLASS"
#endif
#ifdef UE_STRING_CHARTYPE
	#error "UnrealStringIncludes.h.inl should not be included after defining UE_STRING_CHARTYPE"
#endif
#ifdef UE_STRING_CHARTYPE_IS_TCHAR
	#error "UnrealStringIncludes.h.inl should not be included after defining UE_STRING_CHARTYPE_IS_TCHAR"
#endif

#include "AutoRTFM/ReturnFromOpenMode.h"
#include "CoreTypes.h"
#include "Misc/VarArgs.h"
#include "Misc/AssertionMacros.h"
#include "Misc/UEOps.h"
#include "HAL/UnrealMemory.h"
#include "Templates/IsArithmetic.h"
#include "Templates/IsArray.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/UnrealTemplate.h"
#include "Math/NumericLimits.h"
#include "Concepts/Arithmetic.h"
#include "Containers/Array.h"
#include "Misc/CString.h"
#include "Misc/Crc.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/Invoke.h"
#include "Templates/IsValidVariadicFunctionArg.h"
#include "Templates/AndOrNot.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Templates/TypeHash.h"
#include "Templates/IsFloatingPoint.h"
#include "Templates/UnrealTypeTraits.h"
#include "Traits/IsCharType.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "Traits/IsCharEncodingSimplyConvertibleTo.h"
#include "AutoRTFM.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Misc/OutputDevice.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7

#include <type_traits>
