// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Include UnrealString.h.inl's includes before defining the macros, in case the macros 'poison' other headers or there are re-entrant includes.
#include "Containers/UnrealStringIncludes.h.inl" // IWYU pragma: export

// Note that FUtf8String's Printf formatting type is not UTF8CHAR - even though the formatting string must be ANSI,
// the string still supports UTF8CHAR strings as arguments, e.g. FUtf8String::Printf("Name: %s", Utf8Name)
#define UE_STRING_CLASS                        FUtf8String
#define UE_STRING_CHARTYPE                     UTF8CHAR
#define UE_STRING_CHARTYPE_IS_TCHAR            0
#define UE_STRING_PRINTF_FMT_CHARTYPE          ANSICHAR
#define UE_STRING_PRINTF_FMT_CHARTYPE2         UTF8CHAR
#define UE_STRING_DEPRECATED(Version, Message) UE_DEPRECATED(Version, Message)
	#include "Containers/UnrealString.h.inl" // IWYU pragma: export
#undef UE_STRING_DEPRECATED
#undef UE_STRING_PRINTF_FMT_CHARTYPE2
#undef UE_STRING_PRINTF_FMT_CHARTYPE
#undef UE_STRING_CHARTYPE_IS_TCHAR
#undef UE_STRING_CHARTYPE
#undef UE_STRING_CLASS
