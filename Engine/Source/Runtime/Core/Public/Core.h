// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

/*----------------------------------------------------------------------------
	Low level includes.
----------------------------------------------------------------------------*/

// IWYU pragma: begin_exports
#include "CoreMinimal.h"
#include "Containers/ContainersFwd.h"
#include "Misc/Timespan.h"
#include "Misc/DateTime.h"
#include "HAL/PlatformCrt.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "HAL/PlatformMemory.h"
#include "Misc/Char.h"
#include "Templates/EnableIf.h"
#include "Templates/AndOrNot.h"
#include "Templates/IsArithmetic.h"
#include "Templates/IsFloatingPoint.h"
#include "Templates/IsIntegral.h"
#include "Templates/IsPointer.h"
#include "Templates/IsPODType.h"
#include "Templates/IsUECoreType.h"
#include "Templates/IsSigned.h"
#include "Templates/IsTriviallyCopyAssignable.h"
#include "Templates/IsTriviallyCopyConstructible.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "HAL/PlatformMisc.h"
#include "Logging/LogVerbosity.h"
#include "Misc/VarArgs.h"
#include "GenericPlatform/GenericPlatformStricmp.h"
#include "GenericPlatform/GenericPlatformString.h"
#include "HAL/PlatformString.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "HAL/PlatformStackWalk.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "HAL/PlatformMath.h"
#include "GenericPlatform/GenericPlatformNamedPipe.h"
#include "HAL/PlatformNamedPipe.h"
#include "GenericPlatform/GenericPlatformTime.h"
#include "HAL/PlatformTime.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "HAL/PlatformProcess.h"
#include "GenericPlatform/GenericPlatformMutex.h"
#include "HAL/CriticalSection.h"
#include "GenericPlatform/GenericPlatformAtomics.h"
#include "HAL/PlatformAtomics.h"
#include "GenericPlatform/GenericPlatformTLS.h"
#include "HAL/PlatformTLS.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFile.h"
#include "GenericPlatform/GenericPlatformAffinity.h"
#include "HAL/PlatformAffinity.h"
#include "HAL/PlatformIncludes.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/RemoveReference.h"
#include "Templates/IntegralConstant.h"
#include "Templates/IsClass.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Traits/IsContiguousContainer.h"
#include "Misc/CString.h"
#include "Templates/IsEnumClass.h"
#include "GenericPlatform/GenericPlatformProperties.h"
#include "HAL/PlatformProperties.h"
#include "Misc/EngineVersionBase.h"
#include "Internationalization/TextNamespaceFwd.h"
#include "Templates/Less.h"
#include "Templates/Greater.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "UObject/UnrealNames.h"
#include "Misc/OutputDevice.h"
#include "Misc/MessageDialog.h"
#include "Misc/Exec.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/Decay.h"
#include "Templates/Invoke.h"
#include "Templates/Function.h"
#include "Templates/MemoryOps.h"

#include "Misc/CoreDefines.h"

// Container forward declarations
#include "Containers/ContainerAllocationPolicies.h"

#include "UObject/UObjectHierarchyFwd.h"

#include "CoreGlobals.h"

/*----------------------------------------------------------------------------
	Includes.
----------------------------------------------------------------------------*/

#include "HAL/FileManager.h"
#include "ProfilingDebugging/ScopedDebugInfo.h"
#include "Features/IModularFeature.h"
#include "ProfilingDebugging/ExternalProfiler.h"
#include "HAL/MemoryBase.h"
#include "Misc/ByteSwap.h"
#include "Misc/Compression.h"
#include "Misc/StringUtility.h"
#include "Misc/Parse.h"
#include "Containers/StringConv.h"
#include "Misc/Crc.h"
#include "UObject/ObjectVersion.h"
#include "Templates/TypeHash.h"
#include "Containers/EnumAsByte.h"
#include "Serialization/Archive.h"
#include "Serialization/ArchiveProxy.h"
#include "Serialization/NameAsStringProxyArchive.h"
#include "Templates/Sorting.h"
#include "Containers/Array.h"
#include "Containers/ScriptArray.h"
#include "Containers/MRUArray.h"
#include "Containers/IndirectArray.h"
#include "Misc/ITransaction.h"
#include "Containers/ArrayBuilder.h"
#include "Containers/BitArray.h"
#include "Serialization/BitReader.h"
#include "Serialization/BitWriter.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Math/IntPoint.h"
#include "Misc/StructBuilder.h"
#include "Algo/Reverse.h"
#include "HAL/Event.h"
#include "Misc/ScopedEvent.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/SingleThreadEvent.h"
#include "Misc/SingleThreadRunnable.h"
#include "HAL/ThreadManager.h"
#include "Misc/IQueuedWork.h"
#include "Misc/QueuedThreadPool.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/ThreadSafeCounter64.h"
#include "HAL/ThreadSafeBool.h"
#include "Misc/NoopCounter.h"
#include "Misc/ScopeLock.h"
#include "HAL/TlsAutoCleanup.h"
#include "HAL/ThreadSingleton.h"
#include "Containers/ArrayView.h"
#include "Misc/CoreMisc.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Containers/StaticArray.h"
#include "Containers/StaticBitArray.h"
#include "Containers/Set.h"
#include "Containers/Map.h"
#include "Containers/MapBuilder.h"
#include "Containers/List.h"
#include "Containers/ResourceArray.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "Templates/RefCounting.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/ScriptDelegates.h"
#include "Delegates/DelegateSettings.h"
#include "Templates/SharedPointer.h"
#include "Delegates/IDelegateInstance.h"
#include "Delegates/DelegateBase.h"
#include "Delegates/MulticastDelegateBase.h"
#include "Delegates/IntegerSequence.h"
#include "Templates/Tuple.h"
#include "Delegates/Delegate.h"
#include "HAL/ThreadingBase.h"
#include "Internationalization/CulturePointer.h"
#include "Internationalization/TextLocalizationManager.h"
#include "Internationalization/ITextData.h"
#include "Templates/IsArray.h"
#include "Templates/RemoveExtent.h"
#include "Templates/UniquePtr.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextLocalizationManagerGlobals.h"
#include "Templates/UniqueObj.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Culture.h"
#include "Misc/Guid.h"
#include "Misc/NetworkGuid.h"
#include "Math/Color.h"
#include "Math/ColorList.h"
#include "Math/IntVector.h"
#include "Math/Vector2D.h"
#include "Math/IntRect.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Math/VectorRegister.h"
#include "Math/TwoVectors.h"
#include "Math/Edge.h"
#include "Math/Plane.h"
#include "Math/Sphere.h"
#include "Math/CapsuleShape.h"
#include "Math/Rotator.h"
#include "Math/RangeBound.h"
#include "Math/Range.h"
#include "Math/RangeSet.h"
#include "Math/Interval.h"
#include "Math/Box.h"
#include "Math/Box2D.h"
#include "Math/BoxSphereBounds.h"
#include "Math/OrientedBox.h"
#include "Math/Axis.h"
#include "Math/Matrix.h"
#include "Math/RotationTranslationMatrix.h"
#include "Math/RotationAboutPointMatrix.h"
#include "Math/ScaleRotationTranslationMatrix.h"
#include "Math/RotationMatrix.h"
#include "Math/Quat.h"
#include "Math/PerspectiveMatrix.h"
#include "Math/OrthoMatrix.h"
#include "Math/TranslationMatrix.h"
#include "Math/QuatRotationTranslationMatrix.h"
#include "Math/InverseRotationMatrix.h"
#include "Math/ScaleMatrix.h"
#include "Math/MirrorMatrix.h"
#include "Math/ClipProjectionMatrix.h"
#include "Math/InterpCurvePoint.h"
#include "Math/InterpCurve.h"
#include "Math/CurveEdInterface.h"
#include "Math/Float32.h"
#include "Math/Float16.h"
#include "Math/Float16Color.h"
#include "Math/Vector2DHalf.h"
#include "Math/Transform.h"
#include "Math/ConvexHull2d.h"
#include "Math/UnrealMath.h"
#include "Math/SHMath.h"
#include "Math/RandomStream.h"
#include "Logging/LogSuppressionInterface.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "HAL/OutputDevices.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/OutputDeviceNull.h"
#include "Misc/OutputDeviceMemory.h"
#include "Misc/OutputDeviceFile.h"
#include "Misc/OutputDeviceDebug.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/OutputDeviceAnsiError.h"
#include "Misc/BufferedOutputDevice.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Stats/StatsMisc.h"
#include "Containers/LockFreeList.h"
#include "Containers/LockFreeFixedSizeAllocator.h"
#include "Containers/ChunkedArray.h"
#include "Stats/Stats.h"
#include "Misc/CoreStats.h"
#include "Misc/TimeGuard.h"
#include "Misc/MemStack.h"
#include "Async/AsyncWork.h"
#include "Serialization/MemoryArchive.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"
#include "Serialization/BufferReader.h"
#include "Serialization/BufferWriter.h"
#include "Misc/Variant.h"
#include "Misc/WildcardString.h"
#include "Containers/CircularBuffer.h"
#include "Containers/CircularQueue.h"
#include "Containers/Queue.h"
#include "Containers/Ticker.h"
#include "ProfilingDebugging/ProfilingHelpers.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/IConsoleManager.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/SlowTaskStack.h"
#include "Misc/FeedbackContext.h"
#include "Misc/SlowTask.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/AutomationTest.h"
#include "Templates/ScopedCallback.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CallbackDevice.h"
#include "Misc/ObjectThumbnail.h"
#include "Misc/LocalTimestampDirectoryVisitor.h"
#include "Serialization/CustomVersion.h"
#include "UObject/BlueprintsObjectVersion.h"
#include "UObject/BuildObjectVersion.h"
#include "UObject/CoreObjectVersion.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/MobileObjectVersion.h"
#include "UObject/NetworkingObjectVersion.h"
#include "UObject/OnlineObjectVersion.h"
#include "UObject/PhysicsObjectVersion.h"
#include "UObject/PlatformObjectVersion.h"
#include "UObject/RenderingObjectVersion.h"
#include "UObject/SequencerObjectVersion.h"
#include "UObject/VRObjectVersion.h"
#include "Misc/App.h"
#include "Misc/OutputDeviceConsole.h"
#include "Misc/MonitoredProcess.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/EnumRange.h"
// IWYU pragma: end_exports
