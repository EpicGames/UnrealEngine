// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_exports

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "Math/MathFwd.h"
#include "UObject/UObjectHierarchyFwd.h"

// Basic types
class FName;
class FExec;
class FArchive;
class FOutputDevice;
class FFeedbackContext;
struct FDateTime;
struct FGuid;

// Math - See Math/MathFwd.h

// Misc
struct FResourceSizeEx;
class IConsoleVariable;
class FRunnableThread;
class FEvent;
class IPlatformFile;
class FMalloc;
class IFileHandle;
class FAutomationTestBase;
struct FGenericMemoryStats;
class FSHAHash;
class FScriptArray;
class FThreadSafeCounter;
enum class EModuleChangeReason;
struct FManifestContext;
class IConsoleObject;
class FConfigFile;
class FConfigSection;

// Text
class FText;
class FTextFilterString;
enum class ETextFilterTextComparisonMode : uint8;
enum class ETextFilterComparisonOperation : uint8;

// IWYU pragma: end_exports
