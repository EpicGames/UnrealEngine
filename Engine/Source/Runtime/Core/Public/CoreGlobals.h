// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PlatformTLS.h"
#include "Logging/LogMacros.h"
#include "Misc/Build.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/OutputDevice.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Templates/Atomic.h"
#include "UObject/NameTypes.h"

#include <atomic>

class Error;
class FChunkedFixedUObjectArray;
class FConfigCacheIni;
class FExec;
class FFixedUObjectArray;
class FOutputDeviceConsole;
class FOutputDeviceRedirector;
class FRunnableThread;
class FText;
class ITransaction;
class UClass;

CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogHAL, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogSerialization, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogUnrealMath, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogUnrealMatrix, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogContentComparisonCommandlet, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogNetPackageMap, Warning, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogNetSerialization, Warning, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogMemory, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogProfilingDebugging, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogCore, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogOutputDevice, Log, All);

CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogSHA, Warning, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogStats, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogStreaming, Display, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogInit, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogExit, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogExec, Warning, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogScript, Warning, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogLocalization, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogLongPackageNames, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogProcess, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogLoad, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogVirtualization, Log, All);

// Temporary log category, generally you should not check things in that use this
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogTemp, Log, All);

// Platform specific logs, set here to make it easier to use them from anywhere
// need another layer of macro to help using a define in a define
#define DECLARE_LOG_CATEGORY_EXTERN_HELPER(A,B,C) DECLARE_LOG_CATEGORY_EXTERN(A,B,C)
#ifdef PLATFORM_GLOBAL_LOG_CATEGORY
	CORE_API DECLARE_LOG_CATEGORY_EXTERN_HELPER(PLATFORM_GLOBAL_LOG_CATEGORY, Log, All);
#endif
#ifdef PLATFORM_GLOBAL_LOG_CATEGORY_ALT
	CORE_API DECLARE_LOG_CATEGORY_EXTERN_HELPER(PLATFORM_GLOBAL_LOG_CATEGORY_ALT, Log, All);
#endif


CORE_API FOutputDeviceRedirector* GetGlobalLogSingleton();

CORE_API void BootTimingPoint(const ANSICHAR *Message);

CORE_API void DumpBootTiming();

struct FScopedBootTiming
{
	FString Message;
	double StartTime;
	CORE_API FScopedBootTiming(const ANSICHAR *InMessage);
	CORE_API FScopedBootTiming(const ANSICHAR *InMessage, FName Suffix);
	CORE_API ~FScopedBootTiming();
};

struct FEngineTrackedActivityScope
{
	CORE_API FEngineTrackedActivityScope(const TCHAR* Fmt, ...);
	CORE_API FEngineTrackedActivityScope(const FString& Str);
	CORE_API ~FEngineTrackedActivityScope();
};


#define SCOPED_BOOT_TIMING(x) TRACE_CPUPROFILER_EVENT_SCOPE_STR(x); FScopedBootTiming ANONYMOUS_VARIABLE(BootTiming_)(x);
#define UE_SCOPED_ENGINE_ACTIVITY(Fmt, ...) FEngineTrackedActivityScope ANONYMOUS_VARIABLE(EngineActivity_)(Fmt, ## __VA_ARGS__);


#define GLog GetGlobalLogSingleton()
extern CORE_API FConfigCacheIni* GConfig;
extern CORE_API ITransaction* GUndo;
extern CORE_API FOutputDeviceConsole* GLogConsole;
CORE_API extern class FOutputDeviceError*			GError;
CORE_API extern class FFeedbackContext*				GWarn;


extern CORE_API TCHAR GErrorHist[16384];

// #crashReport: 2014-08-19 Combine into one, refactor.
extern CORE_API TCHAR GErrorExceptionDescription[4096];

struct FCoreTexts
{
	const FText& True;
	const FText& False;
	const FText& Yes;
	const FText& No;
	const FText& None;

	static CORE_API const FCoreTexts& Get();

	/** Invalidates existing references. Do not use FCoreTexts after calling. */
	static CORE_API void TearDown();

	// Non-copyable
	FCoreTexts(const FCoreTexts&) = delete;
	FCoreTexts& operator=(const FCoreTexts&) = delete;

	FCoreTexts(const FText& InTrue, const FText& InFalse, const FText& InYes, const FText& InNo, const FText& InNone)
		: True(InTrue), False(InFalse), Yes(InYes), No(InNo), None(InNone)
	{
	}
};

/** If true, this executable is able to run all games (which are loaded as DLL's). */
extern CORE_API bool GIsGameAgnosticExe;

/** When saving out of the game, this override allows the game to load editor only properties. */
extern CORE_API bool GForceLoadEditorOnly;

/** Disable loading of objects not contained within script files; used during script compilation */
extern CORE_API bool GVerifyObjectReferencesOnly;

/** when constructing objects, use the fast path on consoles... */
extern CORE_API bool GFastPathUniqueNameGeneration;

/** allow AActor object to execute script in the editor from specific entry points, such as when running a construction script */
extern CORE_API bool GAllowActorScriptExecutionInEditor;

/** Forces use of template names for newly instanced components in a CDO. */
extern CORE_API bool GCompilingBlueprint;

/** True if we're garbage collecting after a blueprint compilation */
extern CORE_API bool GIsGCingAfterBlueprintCompile;

/** True if we're reconstructing blueprint instances. Should never be true on cooked builds */
extern CORE_API bool GIsReconstructingBlueprintInstances;

/** True if actors and objects are being re-instanced. */
extern CORE_API std::atomic<bool> GIsReinstancing;

/** Helper function to flush resource streaming. */
extern CORE_API void(*GFlushStreamingFunc)(void);

/** The settings used by the UE-as-a-library feature. */
struct FUELibraryOverrideSettings
{
	/** True if we were initialized via the UELibrary.  If this is false,
	    none of the other field values should be acknowledged. */
	bool bIsEmbedded = false;

	/** The window handle to embed the engine into */
	void* WindowHandle = nullptr;

	/** The overridden width of the embedded viewport */
	int32 WindowWidth = 0;

	/** The overridden height of the embedded viewport */
	int32 WindowHeight = 0;
};

/** Settings for when using UE as a library */
extern CORE_API FUELibraryOverrideSettings GUELibraryOverrideSettings;

extern CORE_API bool GIsRunningUnattendedScript;

#if WITH_ENGINE
extern CORE_API bool PRIVATE_GIsRunningCommandlet;
extern CORE_API UClass* PRIVATE_GRunningCommandletClass;

/** If true, initialize RHI and set up scene for rendering even when running a commandlet. */
extern CORE_API bool PRIVATE_GAllowCommandletRendering;

/** If true, initialize audio and even when running a commandlet. */
extern CORE_API bool PRIVATE_GAllowCommandletAudio;
#endif

#if WITH_EDITORONLY_DATA

/**
*	True if we are in the editor.
*	Note that this is still true when using Play In Editor. You may want to use GWorld->HasBegunPlay in that case.
*/
extern CORE_API bool GIsEditor;
extern CORE_API bool GIsImportingT3D;
extern CORE_API bool GIsTransacting;

/** Indicates that the game thread is currently paused deep in a call stack,
while some subset of the editor user interface is pumped.  No game
thread work can be done until the UI pumping loop returns naturally. */
extern CORE_API bool			GIntraFrameDebuggingGameThread;
/** True if this is the first time through the UI message pumping loop. */
extern CORE_API bool			GFirstFrameIntraFrameDebugging;

#elif USING_CODE_ANALYSIS

// Defined as variables during code analysis to prevent lots of '<constant> && <expr>' warnings
extern CORE_API bool GIsEditor;
extern CORE_API bool GIntraFrameDebuggingGameThread;
extern CORE_API bool GFirstFrameIntraFrameDebugging;

#else

#define GIsEditor								false
#define GIntraFrameDebuggingGameThread			false
#define GFirstFrameIntraFrameDebugging			false

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
extern CORE_API bool PRIVATE_GIsRunningCookCommandlet;
extern CORE_API bool PRIVATE_GIsRunningDLCCookCommandlet;
#endif

/**
* Check to see if this executable is running a commandlet (custom command-line processing code in an editor-like environment)
*/
FORCEINLINE bool IsRunningCommandlet()
{
#if WITH_ENGINE
	return PRIVATE_GIsRunningCommandlet;
#else
	return false;
#endif
}

/**
* Check to see if this executable is running the cookcommandlet
*/
FORCEINLINE bool IsRunningCookCommandlet()
{
#if WITH_EDITOR
	return PRIVATE_GIsRunningCookCommandlet;
#else
	return false;
#endif
}

/**
* Check to see if this executable is running the cookcommandlet
*/
FORCEINLINE bool IsRunningDLCCookCommandlet()
{
#if WITH_EDITOR
	return PRIVATE_GIsRunningDLCCookCommandlet;
#else
	return false;
#endif
}

/**
 * Returns the UClass of the currently running commandlet.
 */
FORCEINLINE UClass* GetRunningCommandletClass()
{
#if WITH_ENGINE
	return PRIVATE_GRunningCommandletClass;
#else
	return nullptr;
#endif
}

/**
 * Returns the name of the commandlet that was set on the command line by
 * the arg '-Run=<CommandletName>'.
 * 
 * This should be used by code that needs to know the commandlet name
 * early on during engine startup, before the commandlet has been initialized.
 * If possible, code should prefer calling GetRunningCommandletClass()->GetName()
 * instead.
 * 
 * @return The name of the commandlet that will end up running. If there is
 * no commandlet then an empty string will be returned.
 */
CORE_API FString GetCommandletNameFromCmdline();

/**
 * Check to see if we should initialise RHI and set up scene for rendering even when running a commandlet.
 */
FORCEINLINE bool IsAllowCommandletRendering()
{
#if WITH_ENGINE
	return PRIVATE_GAllowCommandletRendering;
#else
	return false;
#endif
}

FORCEINLINE bool IsAllowCommandletAudio()
{
#if WITH_ENGINE
	return PRIVATE_GAllowCommandletAudio;
#else
	return false;
#endif
}

namespace UE
{
/**
 * If multiple copies of the process are running and communicating together, the director process has MultiprocessId 0
 * and the worker processes have MultiprocessId > 0.
 * When only a single process is running, GetMultiprocessId returns 0.
 * Based on commandline. Primary example is cookworkers in multiprocess cook.
 * Systems that need to write a separate file per running process should use this id as part of their file descriptor.
 */
CORE_API int32 GetMultiprocessId();
}

namespace UE::Private
{
CORE_API void SetMultiprocessId(int32 MultiprocessId);
struct FScopeDisableRunningCookCommandlet
{
	CORE_API FScopeDisableRunningCookCommandlet();
	CORE_API ~FScopeDisableRunningCookCommandlet();
private:
	bool bPrevious = false;
};
}

class FIsDuplicatingClassForReinstancing
{
public:
	CORE_API FIsDuplicatingClassForReinstancing& operator= (bool bOther);
	CORE_API operator bool() const;
};

extern CORE_API bool GEdSelectionLock;
extern CORE_API bool GIsClient;
extern CORE_API bool GIsServer;
extern CORE_API TSAN_ATOMIC(bool) GIsCriticalError;
extern CORE_API TSAN_ATOMIC(bool) GIsRunning;
extern CORE_API FIsDuplicatingClassForReinstancing GIsDuplicatingClassForReinstancing;

/**
* These are set when the engine first starts up.
*/

/**
* This specifies whether the engine was launched as a build machine process.
*/
extern CORE_API bool GIsBuildMachine;

/**
* This determines if we should output any log text.  If Yes then no log text should be emitted.
*/
extern CORE_API bool GIsSilent;

extern CORE_API bool GIsSlowTask;
extern CORE_API bool GSlowTaskOccurred;
extern CORE_API bool GIsGuarded;

/**
* Set this to true to only allow setting RequestingExit at the start of the Engine tick
*   This will remove the chance for undefined behaviour when setting RequestExit
*
* This needs to proved out on all platforms/use cases before this can moved to default
*/
#ifndef UE_SET_REQUEST_EXIT_ON_TICK_ONLY
	#define UE_SET_REQUEST_EXIT_ON_TICK_ONLY 0
#endif

/**
 * Returns true after all initial modules have finished all of their loading phases during Engine startup:
 * Corresponds to the event FCoreDelegates::OnAllModuleLoadingPhasesComplete
 */
extern CORE_API bool IsEngineStartupModuleLoadingComplete();
extern CORE_API void SetEngineStartupModuleLoadingComplete();

UE_DEPRECATED(4.24, "Please use IsEngineExitRequested()/RequestEngineExit(const FString&)")
extern CORE_API bool GIsRequestingExit;

/**
 * This will check if a RequestExit has come in, if it has will set GIsRequestingExit.
 */
extern CORE_API void BeginExitIfRequested();

FORCEINLINE bool IsEngineExitRequested()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GIsRequestingExit;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

// Request that the engine exit as soon as it can safely do so
// The ReasonString is not optional and should be a useful description of why the engine exit is requested
CORE_API void RequestEngineExit(const TCHAR* ReasonString);
CORE_API void RequestEngineExit(const FString& ReasonString);

/**
*	Global value indicating on-screen warnings/message should be displayed.
*	Disabled via console command "DISABLEALLSCREENMESSAGES"
*	Enabled via console command "ENABLEALLSCREENMESSAGES"
*	Toggled via console command "TOGGLEALLSCREENMESSAGES"
*/
extern CORE_API bool GAreScreenMessagesEnabled;
extern CORE_API bool GScreenMessagesRestoreState;

/* Whether we are dumping screen shots */
extern CORE_API int32 GIsDumpingMovie;
extern CORE_API bool GIsHighResScreenshot;
extern CORE_API uint32 GScreenshotResolutionX;
extern CORE_API uint32 GScreenshotResolutionY;
extern CORE_API uint64 GMakeCacheIDIndex;

extern CORE_API FString GEngineIni;

/** Editor ini file locations - stored per engine version (shared across all projects). Migrated between versions on first run. */
extern CORE_API FString GEditorLayoutIni;
extern CORE_API FString GEditorKeyBindingsIni;
extern CORE_API FString GEditorSettingsIni;

/** Editor per-project ini files - stored per project. */
extern CORE_API FString GEditorIni;
extern CORE_API FString GEditorPerProjectIni;

extern CORE_API FString GCompatIni;
extern CORE_API FString GLightmassIni;
extern CORE_API FString GScalabilityIni;
extern CORE_API FString GHardwareIni;
extern CORE_API FString GInputIni;
extern CORE_API FString GGameIni;
extern CORE_API FString GGameUserSettingsIni;
extern CORE_API FString GRuntimeOptionsIni;
extern CORE_API FString GInstallBundleIni;
extern CORE_API FString GDeviceProfilesIni;
extern CORE_API FString GGameplayTagsIni;

extern CORE_API float GNearClippingPlane;
extern CORE_API float GNearClippingPlane_RenderThread;

extern CORE_API bool GExitPurge;
extern CORE_API TCHAR GInternalProjectName[64];
extern CORE_API const TCHAR* GForeignEngineDir;

/** Exec handler for game debugging tool, allowing commands like "editactor" */
extern CORE_API FExec* GDebugToolExec;

/** Whether we're currently in the async loading code path or not */
extern CORE_API bool(*IsAsyncLoading)();

/** Suspends async package loading. */
extern CORE_API void (*SuspendAsyncLoading)();

/** Resumes async package loading. */
extern CORE_API void (*ResumeAsyncLoading)();

/** Suspends async package loading. */
extern CORE_API bool (*IsAsyncLoadingSuspended)();

/** Returns true if async loading is using the async loading thread */
extern CORE_API bool(*IsAsyncLoadingMultithreaded)();

enum class ELoaderType : uint8
{
	NotInitialized,
	LegacyLoader,
	EditorPackageLoader,
	ZenLoader
};

CORE_API const TCHAR* LexToString(ELoaderType Type);

/** Returns the type of the currently active loader, if any. */
extern CORE_API ELoaderType(*GetLoaderType)();

/** Suspends texture updates caused by completed async IOs. */
extern CORE_API void (*SuspendTextureStreamingRenderTasks)();

/** Resume texture updates caused by completed async IOs. */
extern CORE_API void (*ResumeTextureStreamingRenderTasks)();

namespace UE
{
	CORE_API bool GetIsEditorLoadingPackage();
	CORE_API void SetIsEditorLoadingPackage(bool InValue);
}

/** Whether the cooker is currently loading a package or not */
extern CORE_API bool GIsCookerLoadingPackage;

/**
 * Whether GWorld should point to the play in editor world.
 * @warning There are times when GIsPlayInEditorWorld == true and GWorld == nullptr! Notably during LoadMap 
 * during PIE, which can happen when PIE initiates a Browse to another level.
 */
extern CORE_API bool GIsPlayInEditorWorld;

namespace UE
{
	CORE_API int32 GetPlayInEditorID();
	CORE_API void  SetPlayInEditorID(int32 InPlayInEditorID);
}

/** true if the runtime needs textures to be powers of two */
extern CORE_API bool GPlatformNeedsPowerOfTwoTextures;

/** Time at which FPlatformTime::Seconds() was first initialized (very early on) */
extern CORE_API double GStartTime;

/** System time at engine init. */
extern CORE_API FString GSystemStartTime;

/** Whether we are still in the initial loading process. */
extern CORE_API bool GIsInitialLoad;

/* Whether we are using the event driven loader */
UE_DEPRECATED(5.8, "This variable is no longer set and will be removed.")
extern CORE_API bool GEventDrivenLoaderEnabled;

/** true when we are retrieving VTablePtr from UClass */
extern CORE_API bool GIsRetrievingVTablePtr;

/** Steadily increasing frame counter. */
extern CORE_API uint64 GFrameCounter;

extern CORE_API uint64 GFrameCounterRenderThread;

/** GFrameCounter the last time GC was run. */
extern CORE_API uint64 GLastGCFrame;

/** The time input was sampled, in cycles. */
extern CORE_API uint64 GInputTime;

/** Incremented once per frame before the scene is being rendered. In split screen mode this is incremented once for all views (not for each view). */
extern CORE_API uint32 GFrameNumber;

/** NEED TO RENAME, for RT version of GFrameTime use View.ViewFamily->FrameNumber or pass down from RT from GFrameTime). */
extern CORE_API uint32 GFrameNumberRenderThread;

/** Whether we are the first instance of the game running. */
UE_DEPRECATED(5.1, "Please use `FPlatformProcess::IsFirstInstance()`")
extern CORE_API bool GIsFirstInstance;

/** Threshold for a frame to be considered a hitch (in milliseconds). */
extern CORE_API float GHitchThresholdMS;

/** Size to break up data into when saving compressed data */
extern CORE_API int32 GSavingCompressionChunkSize;

/** Thread ID of the main/game thread */
extern CORE_API uint32 GGameThreadId;

/** Thread ID of the render thread, if any */
UE_DEPRECATED(4.26, "Please use `IsInActualRenderingThread()`")
extern CORE_API uint32 GRenderThreadId;

/** Thread ID of the slate thread, if any */
extern CORE_API uint32 GSlateLoadingThreadId;

/** Has GGameThreadId been set yet? */
extern CORE_API bool GIsGameThreadIdInitialized;

/** Determines what kind of trace should occur, NAME_None for none. */
extern CORE_API FLazyName GCurrentTraceName;

/** How to print the time in log output. */
extern CORE_API TSAN_ATOMIC(ELogTimes::Type) GPrintLogTimes;

/** How to print the category in log output. */
extern CORE_API TSAN_ATOMIC(bool) GPrintLogCategory;

/** How to print the verbosity in log output. */
extern CORE_API TSAN_ATOMIC(bool) GPrintLogVerbosity;

#if USE_HITCH_DETECTION
/** Used by the lightweight stats and FGameThreadHitchHeartBeat to print a stat stack for hitches in shipping builds. */
extern CORE_API TSAN_ATOMIC(bool) GHitchDetected;
#endif

/** Whether stats should emit named events for e.g. PIX. */
extern CORE_API int32 GCycleStatsShouldEmitNamedEvents;

/** Whether verbose stats should be also generate external profiler named events.
* Thread sleep/wait stats or extremely high frequency cycle counting stats are disabled by default.
* Has no effect if GCycleStatsShouldEmitNamedEvents is 0.
*/
extern CORE_API bool GShouldEmitVerboseNamedEvents;

/** Disables some warnings and minor features that would interrupt a demo presentation*/
extern CORE_API bool GIsDemoMode;

/** Name of the core package. */
//@Package name transition, remove the double checks 
extern CORE_API FLazyName GLongCorePackageName;
//@Package name transition, remove the double checks 
extern CORE_API FLazyName GLongCoreUObjectPackageName;

/** Whether or not a unit test is currently being run. */
extern CORE_API bool GIsAutomationTesting;

/** Whether or not messages are being pumped outside of main loop */
extern CORE_API bool GPumpingMessagesOutsideOfMainLoop;

/** Whether or not messages are being pumped */
extern CORE_API bool GPumpingMessages;
 
/** Enables various editor and HMD hacks that allow the experimental VR editor feature to work, perhaps at the expense of other systems */
extern CORE_API bool GEnableVREditorHacks;

#if !UE_BUILD_SHIPPING

/** Whether we should ignore the attached debugger. */
extern CORE_API bool GIgnoreDebugger;

#endif // #if !UE_BUILD_SHIPPING

enum class ETaskTag : int32
{
	ENone						= 0 << 0,
	EStaticInit					= 1 << 0,
	EGameThread					= 1 << 1,
	ESlateThread				= 1 << 2,
	// Removed EAudioThread		= 1 << 3,
	ERenderingThread			= 1 << 4,
	ERhiThread					= 1 << 5,
	EAsyncLoadingThread			= 1 << 6,
	EEventThread				= 1 << 7,

	ENamedThreadBits			= (EEventThread << 1) - 1,
	EParallelThread				= 1 << 30, //This can be used when multiple threads or jobs are involved (usually a parallel for) It will avoid the check for uniqueness of the named thread tag.
	EWorkerThread				= 1 << 29 | EParallelThread,
	EParallelRenderingThread	= ERenderingThread | EParallelThread,
	EParallelGameThread			= EGameThread | EParallelThread,
	EParallelRhiThread			= ERhiThread | EParallelThread,
	EParallelLoadingThread		= EAsyncLoadingThread | EParallelThread,
};

ENUM_CLASS_FLAGS(ETaskTag);


/**
 * This class can be used to Tag an execution context aka Thead or Job and allows us to later query the state when we are in the callstack
 * It is usually used for the IsInRendering/GamethreadFunctions.
 *
 * @param CtorSignature InTag the Tag to use
 */
class FTaskTagScope
{
	friend class FRunnableThread;
	friend class FRenderingThread;
	static thread_local ETaskTag ActiveTaskTag;
	static int32 GetStaticThreadId();
	ETaskTag ParentTag;
	ETaskTag Tag;
	bool TagOnlyIfNone;

public:
	/**
	 * Clear the ETaskTag::StaticInit tag so that the Main OS Thread has no active tag.
	 * This will allow functions such as IsInGameThread() to function properly when called from the Main Thread.
	 */
	static void CORE_API SetTagNone();

	/**
	 * Restore the ETaskTag::StaticInit tag so that the destructors of global
	 * (or local static) C++ objects function properly when checking thread
	 * state (from functions like IsInGameThread()).
	 */
	static void CORE_API SetTagStaticInit();

	/**
	* Swap the Tag this is only used when Thread contexts move between different threads.
	*/
	static ETaskTag CORE_API SwapTag(ETaskTag Tag);

public:
	CORE_API FTaskTagScope(ETaskTag InTag);
	CORE_API ~FTaskTagScope();

	static CORE_API ETaskTag GetCurrentTag();
	static CORE_API bool IsCurrentTag(ETaskTag InTag);
	static CORE_API bool IsRunningDuringStaticInit();
};

/**
 * Ensures that current thread is during retrieval of vtable ptr of some
 * UClass.
 *
 * @param CtorSignature The signature of the ctor currently running to
 *		construct proper error message.
 */
CORE_API void EnsureRetrievingVTablePtrDuringCtor(const TCHAR* CtorSignature);

/** @return True if called from the game thread. */
extern CORE_API bool IsInGameThread();

/** @return True if called from the game thread in a parallel for. */
extern CORE_API bool IsInParallelGameThread();

/** @return True if called from a parallel loading thread task. */
extern CORE_API bool IsInParallelLoadingThread();

/** @return True if called from a worker thread. */
extern CORE_API bool IsInWorkerThread();

extern CORE_API bool IsAudioThreadRunning();

/** @return True if called from the audio thread, and not merely a thread calling audio functions. */
extern CORE_API bool IsInAudioThread();

/** @return True if called from the slate thread, and not merely a thread calling slate functions. */
extern CORE_API bool IsInSlateThread();

/** @return True if called from the rendering thread, or if called from ANY thread during single threaded rendering */
extern CORE_API bool IsInRenderingThread();

/** @return True if called from the rendering thread, or if called from ANY thread that isn't the game thread, except that during single threaded rendering the game thread is ok too.*/
extern CORE_API bool IsInParallelRenderingThread();

/** @return True if called from any of the rendering thread. Can be called from any thread without asserting. */
extern CORE_API bool IsInAnyRenderingThread();

/** @return True if called from the rendering thread. */
// Unlike IsInRenderingThread, this will always return false if we are running single threaded. It only returns true if this is actually a separate rendering thread. Mostly useful for checks
extern CORE_API bool IsInActualRenderingThread();

/** @return True if called from the async loading thread if it's enabled, otherwise if called from game thread while is async loading code. */
extern CORE_API bool (*IsInAsyncLoadingThread)();

/** @return True only if called from the async loading thread if it's enabled. */
extern CORE_API bool IsInActualLoadingThread();

/** @return True if RHI thread is running */
extern CORE_API bool IsRHIThreadRunning();

/** @return True if called from the RHI thread, or if called from ANY thread during single threaded rendering */
extern CORE_API bool IsInRHIThread();

/** @return True if called from a thread tagged as an RHI task. */
extern CORE_API bool IsInParallelRHIThread();

extern CORE_API bool GIsRunningRHIInSeparateThread_InternalUseOnly;
extern CORE_API bool GIsRunningRHIInDedicatedThread_InternalUseOnly;
extern CORE_API bool GIsRunningRHIInTaskThread_InternalUseOnly;

/** Boot loading timers */
#if !UE_BUILD_SHIPPING
CORE_API void NotifyLoadingStateChanged(bool bState, const TCHAR *Message);
struct FScopedLoadingState
{
	FString Message;
	FScopedLoadingState(FString&& InMessage)
		: Message(MoveTemp(InMessage))
	{
		NotifyLoadingStateChanged(true, *Message);
	}
	FScopedLoadingState(const FString& InMessage)
		: Message(InMessage)
	{
		NotifyLoadingStateChanged(true, *Message);
	}
	~FScopedLoadingState()
	{
		NotifyLoadingStateChanged(false, *Message);
	}
};
#else
FORCEINLINE void NotifyLoadingStateChanged(bool bState, const TCHAR *Message)
{
}
struct FScopedLoadingState
{
	template <typename T>
	FORCEINLINE FScopedLoadingState(T&& InMessage)
	{
	}
};
#endif

bool CORE_API GetEmitDrawEvents();
void CORE_API SetEmitDrawEvents(bool EmitDrawEvents);

/** Array to help visualize weak pointers in the debugger */
class FChunkedFixedUObjectArray;

extern CORE_API FChunkedFixedUObjectArray* GCoreObjectArrayForDebugVisualizers;

namespace UE::CoreUObject::Private
{
	/** Array to help visualize object paths in the debugger */
	struct FStoredObjectPathDebug;
	
	/** Array to help visualize object handles in the debugger */
	struct FObjectHandlePackageDebugData;

	/** Used to visualize tag values in FAssetData in the debugger */
	struct FStoreBase;
}
namespace UE::Core { struct FVisualizerDebuggingState; }
extern CORE_API UE::CoreUObject::Private::FStoredObjectPathDebug* GCoreComplexObjectPathDebug;
extern CORE_API UE::CoreUObject::Private::FObjectHandlePackageDebugData* GCoreObjectHandlePackageDebug;
extern CORE_API UE::Core::FVisualizerDebuggingState* GCoreDebuggingState;
extern CORE_API UE::CoreUObject::Private::FStoreBase* const* GCoreFixedTagStoresForDebugVisualizers;

/** @return True if running cook-on-the-fly. */
bool CORE_API IsRunningCookOnTheFly();

namespace UE {
	bool CORE_API IsUsingZenPakFileStreaming();
}

/* Used to wrap around null checks with `this` which should be optimized out when not using -fno-delete-null-pointer-checks.
 * The aim is to give the opportunity to the client code to fix any potential causes of `this` being null and stop relying
 * on undefined behavior.
 */
UE_DEPRECATED(5.6, "IsThisNotNull has been deprecated.")
CORE_API FORCENOINLINE bool IsThisNotNull(const void* This, const ANSICHAR* Function);

namespace UE
{
	CORE_API bool IsGCLocked();
}
namespace UE::Private
{
	CORE_API void SetIsGCLockedFunction(bool(*InIsGCLockedFuncPtr)());
}
