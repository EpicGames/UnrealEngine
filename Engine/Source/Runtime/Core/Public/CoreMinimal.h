// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*----------------------------------------------------------------------------
	Low level includes.
----------------------------------------------------------------------------*/

#include "CoreTypes.h"

/*----------------------------------------------------------------------------
	Forward declarations
----------------------------------------------------------------------------*/

#include "CoreFwd.h"
#include "UObject/UObjectHierarchyFwd.h"
#include "Containers/ContainersFwd.h"

/*----------------------------------------------------------------------------
	Commonly used headers
----------------------------------------------------------------------------*/

#include "Misc/VarArgs.h"
#include "Logging/LogVerbosity.h"
#include "Misc/OutputDevice.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "Misc/AssertionMacros.h"
#include "Templates/IsPointer.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformAtomics.h"
#include "Misc/Exec.h"
#include "HAL/MemoryBase.h"
#include "HAL/UnrealMemory.h"
#include "Templates/IsArithmetic.h"
#include "Templates/AndOrNot.h"
#include "Templates/IsPODType.h"
#include "Templates/IsUECoreType.h"
#include "Templates/IsTriviallyCopyConstructible.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/EnableIf.h"
#include "Templates/RemoveReference.h"
#include "Templates/IntegralConstant.h"
#include "Templates/IsClass.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Traits/IsContiguousContainer.h"
#include "Templates/UnrealTemplate.h"
#include "Math/NumericLimits.h"
#include "HAL/PlatformMath.h"
#include "Templates/IsTriviallyCopyAssignable.h"
#include "Templates/MemoryOps.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Templates/IsEnumClass.h"
#include "HAL/PlatformProperties.h"
#include "Misc/EngineVersionBase.h"
#include "Internationalization/TextNamespaceFwd.h"
#include "Serialization/Archive.h"
#include "Templates/Less.h"
#include "Templates/Sorting.h"
#include "Misc/Char.h"
#include "GenericPlatform/GenericPlatformStricmp.h"
#include "GenericPlatform/GenericPlatformString.h"
#include "HAL/PlatformString.h"
#include "Misc/CString.h"
#include "Misc/Crc.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "Misc/FrameNumber.h"
#include "Misc/Timespan.h"
#include "Containers/StringConv.h"
#include "UObject/UnrealNames.h"
#include "UObject/NameTypes.h"
#include "Misc/Parse.h"
#include "Templates/AlignmentTemplates.h"
#include "Misc/StructBuilder.h"
#include "Templates/Decay.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/Invoke.h"
#include "Templates/Function.h"
#include "Templates/TypeHash.h"
#include "Containers/ScriptArray.h"
#include "Containers/BitArray.h"
#include "Containers/SparseArray.h"
#include "Containers/Set.h"
#include "Algo/Reverse.h"
#include "Containers/Map.h"
#include "Math/IntPoint.h"
#include "Math/IntVector.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Vector2D.h"
#include "Math/IntRect.h"
#include "Misc/ByteSwap.h"
#include "Containers/EnumAsByte.h"
#include "HAL/PlatformTLS.h"
#include "CoreGlobals.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/CulturePointer.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Delegates/DelegateSettings.h"
#include "Delegates/IDelegateInstance.h"
#include "Delegates/DelegateBase.h"
#include "Delegates/MulticastDelegateBase.h"
#include "Delegates/IntegerSequence.h"
#include "Templates/Tuple.h"
#include "UObject/ScriptDelegates.h"
#include "Delegates/Delegate.h"
#include "Internationalization/TextLocalizationManager.h"
#include "Misc/Optional.h"
#include "Templates/IsArray.h"
#include "Templates/RemoveExtent.h"
#include "Templates/UniquePtr.h"
#include "Internationalization/Text.h"
#include "Templates/UniqueObj.h"
#include "Internationalization/Internationalization.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Math/VectorRegister.h"
#include "Math/TwoVectors.h"
#include "Math/Edge.h"
#include "UObject/ObjectVersion.h"
#include "Math/CapsuleShape.h"
#include "Math/Rotator.h"
#include "Misc/DateTime.h"
#include "Math/RangeBound.h"
#include "Misc/AutomationEvent.h"
#include "Math/Range.h"
#include "Math/RangeSet.h"
#include "Math/Interval.h"
#include "Math/Box.h"
#include "Math/Box2D.h"
#include "Math/BoxSphereBounds.h"
#include "Math/OrientedBox.h"
#include "Math/Axis.h"
#include "Math/Matrix.h"
#include "Math/RotationTranslationMatrix.h"
#include "Math/RotationAboutPointMatrix.h"
#include "Math/ScaleRotationTranslationMatrix.h"
#include "Math/RotationMatrix.h"
#include "Math/Quat.h"
#include "Math/PerspectiveMatrix.h"
#include "Math/OrthoMatrix.h"
#include "Math/TranslationMatrix.h"
#include "Math/QuatRotationTranslationMatrix.h"
#include "Math/InverseRotationMatrix.h"
#include "Math/ScaleMatrix.h"
#include "Math/MirrorMatrix.h"
#include "Math/ClipProjectionMatrix.h"
#include "Math/Float32.h"
#include "Math/Float16.h"
#include "Math/Transform.h"
#include "Math/ConvexHull2d.h"
#include "Math/UnrealMath.h"
