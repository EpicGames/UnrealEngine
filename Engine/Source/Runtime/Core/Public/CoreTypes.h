// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*----------------------------------------------------------------------------
	Low level includes.
----------------------------------------------------------------------------*/

// IWYU pragma: begin_exports
#include "HAL/Platform.h"
#include "ProfilingDebugging/UMemoryDefines.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/CoreDefines.h"
// IWYU pragma: end_exports

