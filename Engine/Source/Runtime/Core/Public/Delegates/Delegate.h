// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Delegates/MulticastDelegateBase.h" // IWYU pragma: export
#include "Delegates/IntegerSequence.h" // IWYU pragma: export
#include "AutoRTFM.h"

/**
 *  C++ DELEGATES
 *  -----------------------------------------------------------------------------------------------
 *
 *	This system allows you to call member functions on C++ objects in a generic, yet type-safe way.
 *  Using delegates, you can dynamically bind to a member function of an arbitrary object,
 *	then call functions on the object, even if the caller doesn't know the object's type.
 *
 *	The system predefines various combinations of generic function signatures with which you can
 *	declare a delegate type from, filling in the type names for return value and parameters with
 *	whichever types you need.
 *
 *	Both single-cast and multi-cast delegates are supported, as well as "dynamic" delegates which
 *	can be serialized to disk and accessed from blueprints.  Additionally, delegates may define 
 *	"payload" data which will be stored and passed directly to bound functions.
 *
 *
 *
 *  DELEGATE FEATURES
 *  -----------------------------------------------------------------------------------------------
 *
 *	Currently we support delegate signatures using any combination of the following:
 *   		- Functions returning a value
 *			- Up to four "payload" variables
 *   		- Multiple function parameters depending on macro/template declaration
 *   		- Functions declared as 'const'
 *
 *  Multi-cast delegates are also supported, using the 'DECLARE_MULTICAST_DELEGATE...' macros.
 *  Multi-cast delegates allow you to attach multiple function delegates, then execute them all at
 *  once by calling a single "Broadcast()" function.  Multi-cast delegate signatures are not allowed
 *	to use a return value.
 *
 *	Unlike other types, dynamic delegates are integrated into the UObject reflection system and can be
 *	bound to blueprint-implemented functions or serialized to disk. You can also bind native functions,
 *	but the native functions need to be declared with UFUNCTION markup. You do not need to use UFUNCTION for
 *	functions bound to other types of delegates.
 *
 *	You can assign "payload data" to your delegates!  These are arbitrary variables that will be passed
 *  directly to any bound function when it is invoked.  This is really useful as it allows you to store
 *  parameters within the delegate it self at bind-time.  All delegate types (except for "dynamic") supports
 *	payload variables automatically!
 *
 *	When binding to a delegate, you can pass payload data along.  This example passes two custom variables,
 *	a bool and an int32 to a delegate.  Then when the delegate is invoked, these parameters will be passed
 *	to your bound function.  The extra variable arguments must always be accepted after the delegate
 *	type parameter arguments.
 *
 *			MyDelegate.BindStatic( &MyFunction, true, 20 );
 *
 *	Remember to look at the signature table at the bottom of this documentation comment for the macro names 
 *	to use for each function signature type, and the binding table to see options and concerns for binding.
 *
 *
 *
 *  DELEGATES EXAMPLE
 *  -----------------------------------------------------------------------------------------------
 *
 *	Suppose you have a class with a method that you'd like to be able to call from anywhere: 
 *
 *		class FLogWriter
 *		{
 *			void WriteToLog( FString );
 *		};
 *
 *	To call the WriteToLog function, we'll need to create a delegate type for that function's signature.
 *	To do this, you will first declare the delegate using one of the macros below.  For example, here
 *	is a simple delegate type:
 *
 *		DECLARE_DELEGATE_OneParam( FStringDelegate, FString );
 *
 *	This creates a delegate type called 'FStringDelegate' that takes a single parameter of type 'FString'.
 *
 *	Here's an example of how you'd use this 'FStringDelegate' in a class:
 *
 *		class FMyClass
 *		{
 *			FStringDelegate WriteToLogDelegate;
 *		};
 *
 *	This allows your class to hold a pointer to a method in an arbitrary class.  The only thing the
 *	class really knows about this delegate is it's function signature.
 *
 *	Now, to assign the delegate, simply create an instance of your delegate class, passing along the
 *	class that owns the method as a template parameter.  You'll also pass the instance of your object
 *	and the actual function address of the method.  So, here we'll create an instance of our 'FLogWriter'
 *	class, then create a delegate for the 'WriteToLog' method of that object instance:
 *
 *		FSharedRef< FLogWriter > LogWriter( new FLogWriter() );
 *
 *		WriteToLogDelegate.BindSP( LogWriter, &FLogWriter::WriteToLog );
 *
 *	You've just dynamically bound a delegate to a method of a class!  Pretty simple, right?
 *
 *	Note that the 'SP' part of 'BindSP' stands for 'shared pointer', because we're binding to an
 *  object that's owned by a shared pointer.  There are versions for different object types,
 *	such as BindRaw() and BindUObject().  You can bind to global function pointers with BindStatic().
 *
 *	Now, your 'WriteToLog' method can be called by FMyClass without it even knowing anything about
 *	the 'FLogWriter' class!  To call a your delegate, just use the 'Execute()' method:
 *
 *		WriteToLogDelegate.Execute( TEXT( "Delegates are spiffy!" ) );
 *
 *  If you call Execute() before binding a function to the delegate, an assertion will be triggered.  In
 *  many cases, you'll instead want to do this:
 *
 *		WriteToLogDelegate.ExecuteIfBound( TEXT( "Only executes if a function was bound!" ) );
 *
 *	That's pretty much all there is to it!!  You can read below for a bit more information.
 *
 *
 *
 *	MORE INFORMATION
 *  -----------------------------------------------------------------------------------------------
 *
 *	The delegate system understands certain types of objects, and additional features are enabled when
 *  using these objects.  If you bind a delegate to a member of a UObject or shared pointer class, the
 *  delegate system can keep a weak reference to the object, so that if the object gets destroyed out
 *  from underneath the delegate, you'll be able to handle these cases by calling IsBound() or
 *  ExecuteIfBound() functions.  Note the special binding syntax for the various types of supported objects.
 *
 *  It's perfectly safe to copy delegate objects.  Delegates can be passed around by value but this is
 *	generally not recommended since they do have to allocate memory on the heap.  Pass them by reference
 *  when possible!
 *
 *  Delegate signature declarations can exist at global scope, within a namespace or even within a class
 *  declaration (but not function bodies.)
 *
 *
 *
 *  FUNCTION SIGNATURES
 *  -----------------------------------------------------------------------------------------------
 *
 *	Use this table to find the declaration macro to use to declare your delegate.
 *	The full list is defined in DelegateCombinations.h
 *
 *	Function signature									|	Declaration macro
 *  ------------------------------------------------------------------------------------------------------------------------------------------------------------
 *	void Function()										|	DECLARE_DELEGATE( DelegateName )
 *	void Function( <Param1> )							|	DECLARE_DELEGATE_OneParam( DelegateName, Param1Type )
 *	void Function( <Param1>, <Param2> )					|	DECLARE_DELEGATE_TwoParams( DelegateName, Param1Type, Param2Type )
 *	void Function( <Param1>, <Param2>, ... )			|	DECLARE_DELEGATE_<Num>Params( DelegateName, Param1Type, Param2Type, ... )
 *	<RetVal> Function()									|	DECLARE_DELEGATE_RetVal( RetValType, DelegateName )
 *	<RetVal> Function( <Param1> )						|	DECLARE_DELEGATE_RetVal_OneParam( RetValType, DelegateName, Param1Type )
 *	<RetVal> Function( <Param1>, <Param2> )				|	DECLARE_DELEGATE_RetVal_TwoParams( RetValType, DelegateName, Param1Type, Param2Type )
 *	<RetVal> Function( <Param1>, <Param2>, ... )		|	DECLARE_DELEGATE_RetVal_<Num>Params( RetValType, DelegateName, Param1Type, Param2Type, ... )
 *  ------------------------------------------------------------------------------------------------------------------------------------------------------------
 *
 *  Remember, there are three different delegate types you can define (any of the above signatures will work):
 *
 *                       Single-cast delegates:  DECLARE_DELEGATE...()
 *                        Multi-cast delegates:  DECLARE_MULTICAST_DELEGATE...()
 *	 Dynamic (UObject, serializable) delegates:  DECLARE_DYNAMIC_DELEGATE...()
 *
 *
 *
 *	BINDING AND SAFETY
 *  -----------------------------------------------------------------------------------------------
 *
 *	Once a delegate has been declared, it can be bound to functions stored in different places.
 *	Because delegates are often called long after they are bound, extra attention must be paid to 
 *	avoid crashes. This list is for single-cast, for multi-cast delegates, replace Bind in the table 
 *	below with Add. Also for multi-cast delegates, Add will return a handle that can then be used to 
 *	later remove the binding. All multi-cast delegates have an ::FDelegate subtype defining an equivalent
 *	single-cast version, that can be Created one place and then added later.
 *
 *	Bind function										|	Usage
 *  ------------------------------------------------------------------------------------------------------------------------------------------------------------
 *	BindStatic(&GlobalFunctionName)						|	Call a static function, can either be globally scoped or a class static
 *	BindUObject(UObject, &UClass::Function)				|	Call a UObject class member function via a TWeakObjectPtr, will not be called if object is invalid
 *	BindSP(SharedPtr, &FClass::Function)				|	Call a native class member function via a TWeakPtr, will not be called if shared pointer is invalid
 *	BindThreadSafeSP(SharedPtr, &FClass::Function)		|	Call a native class member function via a TWeakPtr, will not be called if shared pointer is invalid
 *	BindRaw(RawPtr, &FClass::Function)					|	Call a native class member function with no safety checks. You MUST call Unbind or Remove when object dies to avoid crashes!
 *	BindLambda(Lambda)									|	Call a lambda function with no safety checks. You MUST make sure all captures will be safe at a later point to avoid crashes!
 *	BindSPLambda(SharedPtr, Lambda)						|	Call a lambda function only if shared pointer is still valid. Captured 'this' will always be valid but any other captures may not be
 *	BindWeakLambda(UObject, Lambda)						|	Call a lambda function only if UObject is still valid. Captured 'this' will always be valid but any other captures may not be
 *	BindUFunction(UObject, FName("FunctionName"))		|	Usable for both native and dynamic delegates, will call a UFUNCTION with specified name
 *	BindDynamic(UObject, &UClass::FunctionName)			|	Convenience wrapper only available for dynamic delegates, FunctionName must be declared as a UFUNCTION
 *  ------------------------------------------------------------------------------------------------------------------------------------------------------------
 *
 */


/** This suffix is appended to all header exported delegates */
#define HEADER_GENERATED_DELEGATE_SIGNATURE_SUFFIX TEXT("__DelegateSignature")

/**
 * Declares a delegate that can only bind to one native function at a time
 *
 * @note: The last parameter is variadic and is used as the 'template args' for this delegate's classes (__VA_ARGS__)
 * @note: To avoid issues with macro expansion breaking code navigation, make sure the type/class name macro params are unique across all of these macros
 */
#define UE_PRIVATE_DECLARE_DELEGATE(DelegateName, FuncType) \
	typedef TDelegate<FuncType> DelegateName;

/** Declares a broadcast delegate that can bind to multiple native functions simultaneously */
#define UE_PRIVATE_DECLARE_MULTICAST_DELEGATE(MulticastDelegateName, FuncType) \
	typedef TMulticastDelegate<FuncType> MulticastDelegateName;

/** Declares a broadcast thread-safe delegate that can bind to multiple native functions simultaneously */
#define UE_PRIVATE_DECLARE_TS_MULTICAST_DELEGATE(MulticastDelegateName, FuncType) \
	typedef TMulticastDelegate<FuncType, FDefaultTSDelegateUserPolicy> MulticastDelegateName;

/**
 * Declares a multicast delegate that is meant to only be activated from OwningType
 *
 * @note: This behavior is not enforced and this type should be considered deprecated for new delegates, use normal multicast instead
 */
#define UE_PRIVATE_DECLARE_EVENT(OwningType, EventName, FuncType) \
	class EventName : public TMulticastDelegate<FuncType> \
	{ \
	};

/** Declares a derived event delegate that works the same as its parent type but is intended to be used by a different owning type */
#define DECLARE_DERIVED_EVENT(OwningType, BaseTypeEvent, EventName) \
	class EventName : public BaseTypeEvent { friend class OwningType; };

/** Declare user's dynamic delegate, with wrapper proxy method for executing the delegate */
#define UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DynamicDelegateClassName, FuncType) \
	class DynamicDelegateClassName : public TDynamicDelegate<FuncType> \
	{ \
	public: \
		using TDynamicDelegate<FuncType>::TDynamicDelegate; \
	};

/** Declare user's dynamic multi-cast delegate, with wrapper proxy method for executing the delegate */
#define UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_DELEGATE(DynamicMulticastDelegateClassName, FuncType) \
	class DynamicMulticastDelegateClassName : public TDynamicMulticastDelegate<FuncType> \
	{ \
	public: \
		using TDynamicMulticastDelegate<FuncType>::TDynamicMulticastDelegate; \
	};

// These macros were an implementation detail and should not be used by user code.  The appropriate DECLARE_* macro in DelegateCombinations.h should be used instead.
#define FUNC_DECLARE_DELEGATE(DelegateName, ReturnType, ...)                            UE_DEPRECATED_MACRO(5.8, "FUNC_DECLARE_DELEGATE has been deprecated.")                   UE_PRIVATE_DECLARE_DELEGATE(DelegateName, ReturnType, ##__VA_ARGS__)
#define FUNC_DECLARE_MULTICAST_DELEGATE(MulticastDelegateName, ReturnType, ...)         UE_DEPRECATED_MACRO(5.8, "FUNC_DECLARE_MULTICAST_DELEGATE has been deprecated.")         UE_PRIVATE_DECLARE_MULTICAST_DELEGATE(MulticastDelegateName, ReturnType, ##__VA_ARGS__)
#define FUNC_DECLARE_TS_MULTICAST_DELEGATE(MulticastDelegateName, ReturnType, ...)      UE_DEPRECATED_MACRO(5.8, "FUNC_DECLARE_TS_MULTICAST_DELEGATE has been deprecated.")      UE_PRIVATE_DECLARE_TS_MULTICAST_DELEGATE(MulticastDelegateName, ReturnType, ##__VA_ARGS__)
#define FUNC_DECLARE_EVENT(OwningType, EventName, ReturnType, ...)                      UE_DEPRECATED_MACRO(5.8, "FUNC_DECLARE_EVENT has been deprecated.")                      UE_PRIVATE_DECLARE_EVENT(OwningType, EventName, ReturnType, ##__VA_ARGS__)
#define FUNC_DECLARE_DYNAMIC_DELEGATE(DynamicDelegateClassName, ...)                    UE_DEPRECATED_MACRO(5.8, "FUNC_DECLARE_DYNAMIC_DELEGATE has been deprecated.")           UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DynamicDelegateClassName, ##__VA_ARGS__)
#define FUNC_DECLARE_DYNAMIC_MULTICAST_DELEGATE(DynamicMulticastDelegateClassName, ...) UE_DEPRECATED_MACRO(5.8, "FUNC_DECLARE_DYNAMIC_MULTICAST_DELEGATE has been deprecated.") UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_DELEGATE(DynamicMulticastDelegateClassName, ##__VA_ARGS__)

namespace UE::Core::Private
{
	template <SIZE_T Len>
	struct TCharArray
	{
		TCharArray() = default;

		constexpr TCharArray(const char(&InStr)[Len])
		{
			for (SIZE_T Index = 0; Index != Len; ++Index)
			{
				this->Str[Index] = InStr[Index];
			}
		}

		char Str[Len];
	};

	// Given a char array as a template argument, returns a new char array that is the substring after the ::
	template <TCharArray CharArray>
	constexpr auto GetTrimmedMemberFunctionName()
	{
		// Returns the substring after the ::
		auto FindFuncName = [](const char* Str)
		{
			while (Str[0] != ':' || Str[1] != ':')
			{
				++Str;
			}
			return Str + 2;
		};

		// Inline version of StrLen that is also constexpr
		auto StrLen = [](const char* Str)
		{
			SIZE_T Result = 0;
			while (*Str != 0)
			{
				++Str;
				++Result;
			}
			return Result;
		};

		constexpr const char* FuncName = FindFuncName(CharArray.Str);
		constexpr SIZE_T FuncNameArrayLen = StrLen(FuncName) + 1;
		TCharArray<FuncNameArrayLen> Result;
		for (SIZE_T Index = 0; Index != FuncNameArrayLen; ++Index)
		{
			Result.Str[Index] = FuncName[Index];
		}
		return Result;
	}

	template <TCharArray CharArray>
	constexpr inline auto TrimmedMemberFunctionName = GetTrimmedMemberFunctionName<CharArray>();
}

#define UE_PRIVATE_STATIC_FUNCTION_FNAME(MemberFunctionName) FName(UE::Core::Private::TrimmedMemberFunctionName<MemberFunctionName>.Str)


/**
 * Helper macro to bind a UObject instance and a member UFUNCTION to a dynamic delegate.
 *
 * @param	UserObject		UObject instance
 * @param	FuncName		Function pointer to member UFUNCTION, usually in form &UClassName::FunctionName
 */
#define BindDynamic( UserObject, FuncName, ... ) __Internal_BindDynamic( UserObject, FuncName, UE_PRIVATE_STATIC_FUNCTION_FNAME( #FuncName ), ##__VA_ARGS__ )

/**
 * Helper macro to bind a UObject instance and a member UFUNCTION to a dynamic multi-cast delegate.
 *
 * @param	UserObject		UObject instance
 * @param	FuncName		Function pointer to member UFUNCTION, usually in form &UClassName::FunctionName
 */
#define AddDynamic( UserObject, FuncName, ... ) __Internal_AddDynamic( UserObject, FuncName, UE_PRIVATE_STATIC_FUNCTION_FNAME( #FuncName ), ##__VA_ARGS__ )

/**
 * Helper macro to bind a UObject instance and a member UFUNCTION to a dynamic multi-cast delegate, but only if it hasn't been bound before.
 *
 * @param	UserObject		UObject instance
 * @param	FuncName		Function pointer to member UFUNCTION, usually in form &UClassName::FunctionName
 */
#define AddUniqueDynamic( UserObject, FuncName, ... ) __Internal_AddUniqueDynamic( UserObject, FuncName, UE_PRIVATE_STATIC_FUNCTION_FNAME( #FuncName ), ##__VA_ARGS__ )

/**
 * Helper macro to unbind a UObject instance and a member UFUNCTION from this multi-cast delegate.
 *
 * @param	UserObject		UObject instance
 * @param	FuncName		Function pointer to member UFUNCTION, usually in form &UClassName::FunctionName
 */
#define RemoveDynamic( UserObject, FuncName, ... ) __Internal_RemoveDynamic( UserObject, FuncName, UE_PRIVATE_STATIC_FUNCTION_FNAME( #FuncName ), ##__VA_ARGS__ )

 /**
  * Helper macro to tests if a UObject instance and a member UFUNCTION are already bound to this multi-cast delegate.
  *
  * @param	UserObject		UObject instance
  * @param	FuncName		Function pointer to member UFUNCTION, usually in form &UClassName::FunctionName
  */
#define IsAlreadyBound( UserObject, FuncName, ... ) __Internal_IsAlreadyBound( UserObject, FuncName, UE_PRIVATE_STATIC_FUNCTION_FNAME( #FuncName ), ##__VA_ARGS__ )


/*********************************************************************************************************************/

// We define this as a guard to prevent DelegateSignatureImpl.inl being included outside of this file
#define __Delegate_h__
#define FUNC_INCLUDING_INLINE_IMPL

#if !UE_BUILD_DOCS
	#include "Delegates/DelegateInstanceInterface.h"
	#include "Delegates/DelegateInstancesImpl.h"
	#include "Delegates/DelegateSignatureImpl.inl" // IWYU pragma: export
	#include "Delegates/DelegateCombinations.h" // IWYU pragma: export
#endif

// No longer allowed to include DelegateSignatureImpl.inl
#undef FUNC_INCLUDING_INLINE_IMPL
#undef __Delegate_h__

/*********************************************************************************************************************/

// Simple delegate used by various utilities such as timers
DECLARE_DELEGATE( FSimpleDelegate );
DECLARE_MULTICAST_DELEGATE( FSimpleMulticastDelegate );
DECLARE_TS_MULTICAST_DELEGATE( FTSSimpleMulticastDelegate );
