// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_exports

#include "CoreTypes.h"
#include "HAL/CriticalSection.h"  // jira SOL-6812: Remove this.
#include "Misc/MTAccessDetector.h"
#include "Misc/ScopeLock.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "AutoRTFM.h"

//#define UE_DETECT_DELEGATES_RACE_CONDITIONS 0

#if !defined(UE_DETECT_DELEGATES_RACE_CONDITIONS)
#define UE_DETECT_DELEGATES_RACE_CONDITIONS ENABLE_MT_DETECTOR
#endif

/**
 * types to specify thread safety mode, an enum would be less readable in debugger
 */
struct FNotThreadSafeNotCheckedDelegateMode;
struct FThreadSafeDelegateMode;
struct FNotThreadSafeDelegateMode;

/**
 * a template for delegates core thread-safety. is supposed to be used as a base class for "empty base optimisation".
 * any access to internal data must happen inside an "read access scope" or "write access scope"
 */
template<typename ThreadSafetyMode>
class TDelegateAccessHandlerBase;

template <typename ThreadSafetyMode>
struct TWriteLockedDelegateAllocation;

// IWYU pragma: end_exports

/**
 * non thread-safe version that does not do any race detection. supposed to be used in a controlled environment that provides own
 * detection or synchronisation.
 */
template<>
class TDelegateAccessHandlerBase<FNotThreadSafeNotCheckedDelegateMode>
{
	friend struct TWriteLockedDelegateAllocation<FNotThreadSafeNotCheckedDelegateMode>;

protected:
	struct FReadAccessScope {};
	struct FWriteAccessScope {};

	[[nodiscard]] FReadAccessScope GetReadAccessScope() const
	{
		return {};
	}

	[[nodiscard]] FWriteAccessScope GetWriteAccessScope()
	{
		return {};
	}
};

template<>
struct TIsZeroConstructType<TDelegateAccessHandlerBase<FNotThreadSafeNotCheckedDelegateMode>>
{
	static constexpr bool Value = true;
};

/**
 * thread-safe version that locks access to the delegate internals.
 * we don't have a recursive RW mutex yet, so "read scope" is actually a write scope. This means that concurrent reads of
 * a thread-safe delegate are mutually exclusive
 */
template<>
class TDelegateAccessHandlerBase<FThreadSafeDelegateMode>
{
	friend struct TWriteLockedDelegateAllocation<FThreadSafeDelegateMode>;

protected:
	struct FReadAccessScope { UE::TScopeLock<FTransactionallySafeCriticalSection> Lock; };
	struct FWriteAccessScope { UE::TScopeLock<FTransactionallySafeCriticalSection> Lock; };

	[[nodiscard]] FReadAccessScope GetReadAccessScope() const
	{
		return FReadAccessScope{UE::TScopeLock{Mutex}};
	}

	[[nodiscard]] FWriteAccessScope GetWriteAccessScope()
	{
		return FWriteAccessScope{UE::TScopeLock{Mutex}};
	}

private:
	mutable FTransactionallySafeCriticalSection Mutex;
};

template<>
struct TIsZeroConstructType<TDelegateAccessHandlerBase<FThreadSafeDelegateMode>>
{
	static constexpr bool Value = false;
};

#if UE_DETECT_DELEGATES_RACE_CONDITIONS

/**
 * non thread-safe version that detects not thread-safe delegates used concurrently (dev builds only)
 */
template<>
class TDelegateAccessHandlerBase<FNotThreadSafeDelegateMode>
{
	friend struct TWriteLockedDelegateAllocation<FNotThreadSafeDelegateMode>;

protected:
#if !UE_AUTORTFM
	class FReadAccessScope
	{
	public:
		UE_NONCOPYABLE(FReadAccessScope);

		explicit FReadAccessScope(const FMRSWRecursiveAccessDetector& InAccessDetector)
			: DestructionSentinel(FMRSWRecursiveAccessDetector::EAccessType::Reader)
		{
			DestructionSentinel.Accessor = &InAccessDetector;
			DestructionSentinel.Accessor->AcquireReadAccess(DestructionSentinel);
		}

		~FReadAccessScope()
		{
			if (!DestructionSentinel.bDestroyed) // only if `AccessDetector` wasn't destroyed while being accessed
			{
				DestructionSentinel.Accessor->ReleaseReadAccess(DestructionSentinel);
			}
		}

	private:
		FMRSWRecursiveAccessDetector::FDestructionSentinel DestructionSentinel;
	};

	[[nodiscard]] FReadAccessScope GetReadAccessScope() const
	{
		return FReadAccessScope(AccessDetector);
	}

	class FWriteAccessScope
	{
	public:
		UE_NONCOPYABLE(FWriteAccessScope);

		explicit FWriteAccessScope(FMRSWRecursiveAccessDetector& InAccessDetector)
			: DestructionSentinel(FMRSWRecursiveAccessDetector::EAccessType::Writer)
		{
			DestructionSentinel.Accessor = &InAccessDetector;
			InAccessDetector.AcquireWriteAccess(DestructionSentinel);
		}

		~FWriteAccessScope()
		{
			if (!DestructionSentinel.bDestroyed) // only if `AccessDetector` wasn't destroyed while being accessed
			{
				// the `const_cast` doesn't violate constness here as we got a mutable accessor in `FWriteAccessScope` constuctor,
				// otherwise we'd need to have a redundant ptr to a mutable accessor as a member
				const_cast<FMRSWRecursiveAccessDetector*>(DestructionSentinel.Accessor)->ReleaseWriteAccess(DestructionSentinel);
			}
		}

	private:
		FMRSWRecursiveAccessDetector::FDestructionSentinel DestructionSentinel;
	};

	[[nodiscard]] FWriteAccessScope GetWriteAccessScope()
	{
		return FWriteAccessScope(AccessDetector);
	}
#else
	struct FReadAccessScope {};
	struct FWriteAccessScope {};

	[[nodiscard]] FReadAccessScope GetReadAccessScope() const
	{
		return {};
	}

	[[nodiscard]] FWriteAccessScope GetWriteAccessScope()
	{
		return {};
	}
#endif

private:
	FMRSWRecursiveAccessDetector AccessDetector;
};

#else // UE_DETECT_DELEGATES_RACE_CONDITIONS

template<>
class TDelegateAccessHandlerBase<FNotThreadSafeDelegateMode> : public TDelegateAccessHandlerBase<FNotThreadSafeNotCheckedDelegateMode>
{
	friend struct TWriteLockedDelegateAllocation<FNotThreadSafeDelegateMode>;
};

#endif // UE_DETECT_DELEGATES_RACE_CONDITIONS

template<>
struct TIsZeroConstructType<TDelegateAccessHandlerBase<FNotThreadSafeDelegateMode>>
{
	static constexpr bool Value = true;
};
