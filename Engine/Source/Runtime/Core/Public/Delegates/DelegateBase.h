// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Math/UnrealMathUtility.h"
#include "UObject/NameTypes.h"
#include "Delegates/DelegateAccessHandler.h"
#include "Delegates/DelegateInstancesImplFwd.h"
#include "Delegates/DelegateSettings.h"
#include "Delegates/IDelegateInstance.h"

#if !defined(_WIN32) || defined(_WIN64) || (defined(ALLOW_DELEGATE_INLINE_ALLOCATORS_ON_WIN32) && ALLOW_DELEGATE_INLINE_ALLOCATORS_ON_WIN32)
	using FAlignedInlineDelegateType = TAlignedBytes<16, 16>;
	#if !defined(NUM_DELEGATE_INLINE_BYTES) || NUM_DELEGATE_INLINE_BYTES == 0
		using FDelegateAllocatorType = FHeapAllocator;
	#elif NUM_DELEGATE_INLINE_BYTES < 0 || (NUM_DELEGATE_INLINE_BYTES % 16) != 0
		#error NUM_DELEGATE_INLINE_BYTES must be a multiple of 16
	#else
		using FDelegateAllocatorType = TInlineAllocator<(NUM_DELEGATE_INLINE_BYTES / 16)>;
	#endif
#else
	// ... except on Win32, because we can't pass 16-byte aligned types by value, as some delegates are
	// so we'll just keep it heap-allocated, which are always sufficiently aligned.
	using FAlignedInlineDelegateType = TAlignedBytes<16, 8>;
	using FDelegateAllocatorType     = FHeapAllocator;
#endif

template <typename UserPolicy>
class TMulticastDelegateBase;

template <typename UserPolicy>
class TDelegateBase;

ALIAS_TEMPLATE_TYPE_LAYOUT(template<typename ElementType>, FDelegateAllocatorType::ForElementType<ElementType>, void*);

// not thread-safe version, with automatic race detection in dev builds
struct FDefaultDelegateUserPolicy
{
	// To extend delegates, you should implement a policy struct like this and pass it as the second template
	// argument to TDelegate and TMulticastDelegate.  This policy struct containing three classes called:
	// 
	// FDelegateInstanceExtras:
	//   - Must publicly inherit IDelegateInstance.
	//   - Should contain any extra data and functions injected into a binding (the object which holds and
	//     is able to invoke the binding passed to FMyDelegate::CreateSP, FMyDelegate::CreateLambda etc.).
	//   - This binding is not available through the public API of the delegate, but is accessible to FDelegateExtras.
	//
	// FDelegateExtras:
	//   - Must publicly inherit TDelegateBase<FThreadSafetyMode>.
	//   - Should contain any extra data and functions injected into a delegate (the object which holds an
	//     FDelegateInstance-derived object, above).
	//   - Public data members and member functions are accessible directly through the TDelegate object.
	//   - Typically member functions in this class will forward calls to the inner FDelegateInstanceExtras,
	//     by downcasting the result of a call to GetDelegateInstanceProtected().
	//
	// FMulticastDelegateExtras:
	//   - Must publicly inherit TMulticastDelegateBase<FYourUserPolicyStruct>.
	//   - Should contain any extra data and functions injected into a multicast delegate (the object which
	//     holds an array of FDelegateExtras-derived objects which is the invocation list).
	//   - Public data members and member functions are accessible directly through the TMulticastDelegate object.

	using FDelegateInstanceExtras = IDelegateInstance;

	using FThreadSafetyMode =
#if UE_DETECT_DELEGATES_RACE_CONDITIONS
		FNotThreadSafeDelegateMode;
#else
		FNotThreadSafeNotCheckedDelegateMode;
#endif
	using FDelegateExtras = TDelegateBase<FThreadSafetyMode>;
	using FMulticastDelegateExtras = TMulticastDelegateBase<FDefaultDelegateUserPolicy>;
};

// thread-safe version
struct FDefaultTSDelegateUserPolicy
{
	// see `FDefaultDelegateUserPolicy` for documentation

	using FDelegateInstanceExtras = IDelegateInstance;

	using FThreadSafetyMode = FThreadSafeDelegateMode;
	using FDelegateExtras = TDelegateBase<FThreadSafetyMode>;
	using FMulticastDelegateExtras = TMulticastDelegateBase<FDefaultTSDelegateUserPolicy>;
};

// not thread-safe version, no race detection. used primarily for deprecated unsafe delegates that must be kept running for backward compatibility
struct FNotThreadSafeNotCheckedDelegateUserPolicy
{
	using FDelegateInstanceExtras = IDelegateInstance;

	using FThreadSafetyMode = FNotThreadSafeNotCheckedDelegateMode;
	using FDelegateExtras = TDelegateBase<FThreadSafetyMode>;
	using FMulticastDelegateExtras = TMulticastDelegateBase<FNotThreadSafeNotCheckedDelegateUserPolicy>;
};

struct FDelegateAllocation
{
	FDelegateAllocatorType::ForElementType<FAlignedInlineDelegateType> DelegateAllocator;
	int32 DelegateSize = 0;
};

template <typename ThreadSafetyMode>
struct TWriteLockedDelegateAllocation;
struct FPrivateDelegateAllocation;

template <typename ThreadSafetyMode>
void* operator new(size_t Size, const TWriteLockedDelegateAllocation<ThreadSafetyMode>& LockedAllocation);
void* operator new(size_t Size, const FPrivateDelegateAllocation& Allocation);

template <typename ThreadSafetyMode>
struct TWriteLockedDelegateAllocation
{
	UE_NONCOPYABLE(TWriteLockedDelegateAllocation)

	friend void* operator new<ThreadSafetyMode>(size_t Size, const TWriteLockedDelegateAllocation<ThreadSafetyMode>& LockedAllocation);

	explicit TWriteLockedDelegateAllocation(TDelegateBase<ThreadSafetyMode>& Delegate)
		: WriteScope(Delegate.GetWriteAccessScope())
		, Allocation(Delegate)
	{
	}

private:
	typename TDelegateAccessHandlerBase<ThreadSafetyMode>::FWriteAccessScope WriteScope;
	FDelegateAllocation& Allocation;
};

/**
 * Used for delegate allocations when the underlying allocation is known to be not (yet)
 * visible to other threads so locking is unnecessary. (Mainly when the target is
 * a temporary local variable.)
 */
struct FPrivateDelegateAllocation
{
	UE_NONCOPYABLE(FPrivateDelegateAllocation);

	friend void* operator new(size_t Size, const FPrivateDelegateAllocation& Allocation);

	template <typename ThreadSafetyMode>
	explicit FPrivateDelegateAllocation(TDelegateBase<ThreadSafetyMode>& Delegate)
		: Allocation(Delegate)
	{
	}

private:
	FDelegateAllocation& Allocation;
};

/**
 * Base class for unicast delegates.
 */
template<typename ThreadSafetyMode>
class TDelegateBase : public TDelegateAccessHandlerBase<ThreadSafetyMode>, private FDelegateAllocation
{
	friend struct TWriteLockedDelegateAllocation<ThreadSafetyMode>;
	friend struct FPrivateDelegateAllocation;

private:
	using Super = TDelegateAccessHandlerBase<ThreadSafetyMode>;

	template <typename>
	friend class TMulticastDelegateBase;

	template <typename>
	friend class TDelegateBase;

	template <class, typename, typename, typename...>
	friend class TBaseUFunctionDelegateInstance;

	template <bool, class, ESPMode, typename, typename, typename...>
	friend class TBaseSPMethodDelegateInstance;

	template <ESPMode, typename, typename, typename, typename...>
	friend class TBaseSPLambdaDelegateInstance;

	template <bool, class, typename, typename, typename...>
	friend class TBaseRawMethodDelegateInstance;

	template <bool, class, typename, typename, typename...>
	friend class TBaseUObjectMethodDelegateInstance;

	template <typename, typename, typename...>
	friend class TBaseStaticDelegateInstance;

	template <typename, typename, typename, typename...>
	friend class TBaseFunctorDelegateInstance;

	template <typename, typename, typename, typename...>
	friend class TWeakBaseFunctorDelegateInstance;

	friend class FWriteLockedDelegateAllocation;

protected:
	using typename Super::FReadAccessScope;
	using Super::GetReadAccessScope;
	using typename Super::FWriteAccessScope;
	using Super::GetWriteAccessScope;

	explicit TDelegateBase() = default;

public:
	~TDelegateBase()
	{
		Unbind();
	}

	TDelegateBase(TDelegateBase&& Other)
	{
		MoveConstruct(MoveTemp(Other));
	}

	TDelegateBase& operator=(TDelegateBase&& Other)
	{
		MoveAssign(MoveTemp(Other));
		return *this;
	}

	// support for moving from delegates with different thread-safety mode
	template<typename OtherThreadSafetyMode>
	explicit TDelegateBase(TDelegateBase<OtherThreadSafetyMode>&& Other)
	{
		MoveConstruct(MoveTemp(Other));
	}

	/**
	 * Unbinds this delegate
	 */
	inline void Unbind()
	{
		FWriteAccessScope WriteScope = GetWriteAccessScope();

		UnbindUnchecked();
	}

	/**
	 * Returns the amount of memory allocated by this delegate, not including sizeof(*this).
	 */
	SIZE_T GetAllocatedSize() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		return DelegateAllocator.GetAllocatedSize(DelegateSize, sizeof(FAlignedInlineDelegateType));
	}

#if USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME

	/**
	 * Tries to return the name of a bound function.  Returns NAME_None if the delegate is unbound or
	 * a binding name is unavailable.
	 *
	 * Note: Only intended to be used to aid debugging of delegates.
	 *
	 * @return The name of the bound function, NAME_None if no name was available.
	 */
	FName TryGetBoundFunctionName() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		const IDelegateInstance* DelegateInstance = GetDelegateInstanceProtected();
		return DelegateInstance ? DelegateInstance->TryGetBoundFunctionName() : NAME_None;
	}

#endif

	/**
	 * If this is a UFunction or UObject delegate, return the UObject.
	 *
	 * @return The object associated with this delegate if there is one.
	 */
	inline class UObject* GetUObject( ) const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		const IDelegateInstance* DelegateInstance = GetDelegateInstanceProtected();
		return DelegateInstance ? DelegateInstance->GetUObject() : nullptr;
	}

	/**
	 * Checks to see if the user object bound to this delegate is still valid.
	 *
	 * @return True if the user object is still valid and it's safe to execute the function call.
	 */
	inline bool IsBound( ) const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		const IDelegateInstance* DelegateInstance = GetDelegateInstanceProtected();
		return DelegateInstance && DelegateInstance->IsSafeToExecute();
	}

	/** 
	 * Returns a pointer to an object bound to this delegate, intended for quick lookup in the timer manager,
	 *
	 * @return A pointer to an object referenced by the delegate.
	 */
	inline const void* GetObjectForTimerManager() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		const IDelegateInstance* DelegateInstance = GetDelegateInstanceProtected();
		return DelegateInstance ? DelegateInstance->GetObjectForTimerManager() : nullptr;
	}

	/**
	 * Returns the address of the method pointer which can be used to learn the address of the function that will be executed.
	 * Returns nullptr if this delegate type does not directly invoke a function pointer.
	 *
	 * Note: Only intended to be used to aid debugging of delegates.
	 *
	 * @return The address of the function pointer that would be executed by this delegate
	 */
	uint64 GetBoundProgramCounterForTimerManager() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		const IDelegateInstance* DelegateInstance = GetDelegateInstanceProtected();
		return DelegateInstance ? DelegateInstance->GetBoundProgramCounterForTimerManager() : 0;
	}

	/** 
	 * Checks to see if this delegate is bound to the given user object.
	 *
	 * @return True if this delegate is bound to InUserObject, false otherwise.
	 */
	inline bool IsBoundToObject(FDelegateUserObjectConst InUserObject) const
	{
		if (!InUserObject)
		{
			return false;
		}

		FReadAccessScope ReadScope = GetReadAccessScope();

		const IDelegateInstance* DelegateInstance = GetDelegateInstanceProtected();
		return DelegateInstance && DelegateInstance->HasSameObject(InUserObject);
	}

	/** 
	 * Checks to see if this delegate can ever become valid again - if not, it can
	 * be removed from broadcast lists or otherwise repurposed.
	 *
	 * @return True if the delegate is compatable, false otherwise.
	 */
	inline bool IsCompactable() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		const IDelegateInstance* DelegateInstance = GetDelegateInstanceProtected();
		return !DelegateInstance || DelegateInstance->IsCompactable();
	}

	/**
	 * Gets a handle to the delegate.
	 *
	 * @return The delegate instance.
	 */
	inline FDelegateHandle GetHandle() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		const IDelegateInstance* DelegateInstance = GetDelegateInstanceProtected();
		return DelegateInstance ? DelegateInstance->GetHandle() : FDelegateHandle{};
	}

protected:
	/**
	 * Gets the delegate instance.  Not intended for use by user code.
	 *
	 * @return The delegate instance.
	 * @see SetDelegateInstance
	 */
	UE_FORCEINLINE_HINT IDelegateInstance* GetDelegateInstanceProtected()
	{
		return DelegateSize ? (IDelegateInstance*)DelegateAllocator.GetAllocation() : nullptr;
	}

	UE_FORCEINLINE_HINT const IDelegateInstance* GetDelegateInstanceProtected() const
	{
		return DelegateSize ? (const IDelegateInstance*)DelegateAllocator.GetAllocation() : nullptr;
	}

private:
	template<typename OtherThreadSafetyMode>
	void MoveConstruct(TDelegateBase<OtherThreadSafetyMode>&& Other)
	{
		typename TDelegateBase<OtherThreadSafetyMode>::FWriteAccessScope OtherWriteScope = Other.GetWriteAccessScope();

		DelegateAllocator.MoveToEmpty(Other.DelegateAllocator);
		DelegateSize = Other.DelegateSize;
		Other.DelegateSize = 0;
	}

	template<typename OtherThreadSafetyMode>
	void MoveAssign(TDelegateBase<OtherThreadSafetyMode>&& Other)
	{
		FDelegateAllocatorType::ForElementType<FAlignedInlineDelegateType> OtherDelegateAllocator;
		int32 OtherDelegateSize;
		{
			typename TDelegateBase<OtherThreadSafetyMode>::FWriteAccessScope OtherWriteScope = Other.GetWriteAccessScope();
			OtherDelegateAllocator.MoveToEmpty(Other.DelegateAllocator);
			OtherDelegateSize = Other.DelegateSize;
			Other.DelegateSize = 0;
		}

		{
			FWriteAccessScope WriteScope = GetWriteAccessScope();

			UnbindUnchecked();
			DelegateAllocator.MoveToEmpty(OtherDelegateAllocator);
			DelegateSize = OtherDelegateSize;
		}
	}

private:
	inline void UnbindUnchecked()
	{
		if (IDelegateInstance* Ptr = GetDelegateInstanceProtected())
		{
			Ptr->~IDelegateInstance();
			DelegateAllocator.ResizeAllocation(0, 0, sizeof(FAlignedInlineDelegateType));
			DelegateSize = 0;
		}
	}

	using FDelegateAllocation::DelegateAllocator;
	using FDelegateAllocation::DelegateSize;
};

namespace UE::Core::Private
{
	// Should only be called by the TWriteLockedDelegateAllocation<ThreadSafetyMode> overload, because it obtains the write lock
	CORE_API void* DelegateAllocate(size_t Size, FDelegateAllocation& Allocation);
}

template <typename ThreadSafetyMode>
UE_FORCEINLINE_HINT void* operator new(size_t Size, const TWriteLockedDelegateAllocation<ThreadSafetyMode>& LockedAllocation)
{
	return UE::Core::Private::DelegateAllocate(Size, LockedAllocation.Allocation);
}

template <typename ThreadSafetyMode>
UE_FORCEINLINE_HINT void operator delete(void*, const TWriteLockedDelegateAllocation<ThreadSafetyMode>& LockedAllocation)
{
}

UE_FORCEINLINE_HINT void* operator new(size_t Size, const FPrivateDelegateAllocation& Allocation)
{
	return UE::Core::Private::DelegateAllocate(Size, Allocation.Allocation);
}

UE_FORCEINLINE_HINT void operator delete(void*, const FPrivateDelegateAllocation& Allocation)
{
}
