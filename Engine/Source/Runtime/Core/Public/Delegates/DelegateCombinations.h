// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 *  DELEGATE PARAMETER COMBINATIONS
 *  -----------------------------------------------------------------------------------------------
 *
 *	This file defines the different combinations of macros used to declare different delegate types.
 *	Each delegate type can be bound to functions that take up to 9 parameters.
 *	Full documentation for the delegate system can be found in the main Delegate.h header, 
 *	which should be included in cpp files that need access to these macros.
 */

#if 0 // Example of correct header include
#include "Delegates/Delegate.h" // HEADER_UNIT_IGNORE
#endif

/** Declares a delegate that can only bind to one native function at a time */
#define DECLARE_DELEGATE(DelegateName) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, void())

/** Declares a broadcast delegate that can bind to multiple native functions simultaneously */
#define DECLARE_MULTICAST_DELEGATE(DelegateName) UE_PRIVATE_DECLARE_MULTICAST_DELEGATE(DelegateName, void())

/** Declares a broadcast thread-safe delegate that can bind to multiple native functions simultaneously */
#define DECLARE_TS_MULTICAST_DELEGATE(DelegateName) UE_PRIVATE_DECLARE_TS_MULTICAST_DELEGATE(DelegateName, void())

/**
 * Declares a multicast delegate that is meant to only be activated from OwningType
 * NOTE: This behavior is not enforced and this type should be considered deprecated for new delegates, use normal multicast instead
 */
#define DECLARE_EVENT(OwningType, EventName) UE_PRIVATE_DECLARE_EVENT(OwningType, EventName, void())

/** Declares a blueprint-accessible delegate that can only bind to one UFUNCTION at a time */
#define DECLARE_DYNAMIC_DELEGATE(DelegateName) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, void())

/** Declares a blueprint-accessible broadcast delegate that can bind to multiple native UFUNCTIONs simultaneously */
#define DECLARE_DYNAMIC_MULTICAST_DELEGATE(DelegateName) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_DELEGATE(DelegateName, void())

/** Declares a delegate with return value that can only bind to one native function at a time */
#define DECLARE_DELEGATE_RetVal(ReturnValueType, DelegateName) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, ReturnValueType())

/** Declares a blueprint-accessible delegate with return value that can only bind to one UFUNCTION at a time */
#define DECLARE_DYNAMIC_DELEGATE_RetVal(ReturnValueType, DelegateName) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, ReturnValueType())


// Multiple-parameter versions of above delegate types:
#define DECLARE_DELEGATE_OneParam(DelegateName, Param1Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, void(Param1Type))
#define DECLARE_MULTICAST_DELEGATE_OneParam(DelegateName, Param1Type) UE_PRIVATE_DECLARE_MULTICAST_DELEGATE(DelegateName, void(Param1Type))
#define DECLARE_TS_MULTICAST_DELEGATE_OneParam(DelegateName, Param1Type) UE_PRIVATE_DECLARE_TS_MULTICAST_DELEGATE(DelegateName, void(Param1Type))
#define DECLARE_EVENT_OneParam(OwningType, EventName, Param1Type) UE_PRIVATE_DECLARE_EVENT(OwningType, EventName, void(Param1Type))
#define DECLARE_DYNAMIC_DELEGATE_OneParam(DelegateName, Param1Type, Param1Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, void(Param1Type Param1Name))
#define DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(DelegateName, Param1Type, Param1Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_DELEGATE(DelegateName, void(Param1Type Param1Name))
#define DECLARE_DELEGATE_RetVal_OneParam(ReturnValueType, DelegateName, Param1Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, ReturnValueType(Param1Type))
#define DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(ReturnValueType, DelegateName, Param1Type, Param1Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, ReturnValueType(Param1Type Param1Name))

#define DECLARE_DELEGATE_TwoParams(DelegateName, Param1Type, Param2Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, void(Param1Type, Param2Type))
#define DECLARE_MULTICAST_DELEGATE_TwoParams(DelegateName, Param1Type, Param2Type) UE_PRIVATE_DECLARE_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type))
#define DECLARE_TS_MULTICAST_DELEGATE_TwoParams(DelegateName, Param1Type, Param2Type) UE_PRIVATE_DECLARE_TS_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type))
#define DECLARE_EVENT_TwoParams(OwningType, EventName, Param1Type, Param2Type) UE_PRIVATE_DECLARE_EVENT(OwningType, EventName, void(Param1Type, Param2Type))
#define DECLARE_DYNAMIC_DELEGATE_TwoParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name))
#define DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name))
#define DECLARE_DELEGATE_RetVal_TwoParams(ReturnValueType, DelegateName, Param1Type, Param2Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, ReturnValueType(Param1Type, Param2Type))
#define DECLARE_DYNAMIC_DELEGATE_RetVal_TwoParams(ReturnValueType, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, ReturnValueType(Param1Type Param1Name, Param2Type Param2Name))

#define DECLARE_DELEGATE_ThreeParams(DelegateName, Param1Type, Param2Type, Param3Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type))
#define DECLARE_MULTICAST_DELEGATE_ThreeParams(DelegateName, Param1Type, Param2Type, Param3Type) UE_PRIVATE_DECLARE_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type))
#define DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(DelegateName, Param1Type, Param2Type, Param3Type) UE_PRIVATE_DECLARE_TS_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type))
#define DECLARE_EVENT_ThreeParams(OwningType, EventName, Param1Type, Param2Type, Param3Type) UE_PRIVATE_DECLARE_EVENT(OwningType, EventName, void(Param1Type, Param2Type, Param3Type))
#define DECLARE_DYNAMIC_DELEGATE_ThreeParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name))
#define DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name))
#define DECLARE_DELEGATE_RetVal_ThreeParams(ReturnValueType, DelegateName, Param1Type, Param2Type, Param3Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, ReturnValueType(Param1Type, Param2Type, Param3Type))
#define DECLARE_DYNAMIC_DELEGATE_RetVal_ThreeParams(ReturnValueType, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, ReturnValueType(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name))

#define DECLARE_DELEGATE_FourParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type) UE_PRIVATE_DECLARE_DELEGATE( DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type))
#define DECLARE_MULTICAST_DELEGATE_FourParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type) UE_PRIVATE_DECLARE_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type))
#define DECLARE_TS_MULTICAST_DELEGATE_FourParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type) UE_PRIVATE_DECLARE_TS_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type))
#define DECLARE_EVENT_FourParams(OwningType, EventName, Param1Type, Param2Type, Param3Type, Param4Type) UE_PRIVATE_DECLARE_EVENT(OwningType, EventName, void(Param1Type, Param2Type, Param3Type, Param4Type))
#define DECLARE_DYNAMIC_DELEGATE_FourParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name))
#define DECLARE_DYNAMIC_MULTICAST_DELEGATE_FourParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name))
#define DECLARE_DELEGATE_RetVal_FourParams(ReturnValueType, DelegateName, Param1Type, Param2Type, Param3Type, Param4Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, ReturnValueType(Param1Type, Param2Type, Param3Type, Param4Type))
#define DECLARE_DYNAMIC_DELEGATE_RetVal_FourParams(ReturnValueType, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, ReturnValueType(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name))

#define DECLARE_DELEGATE_FiveParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type))
#define DECLARE_MULTICAST_DELEGATE_FiveParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type) UE_PRIVATE_DECLARE_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type))
#define DECLARE_TS_MULTICAST_DELEGATE_FiveParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type) UE_PRIVATE_DECLARE_TS_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type))
#define DECLARE_EVENT_FiveParams(OwningType, EventName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type) UE_PRIVATE_DECLARE_EVENT(OwningType, EventName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type))
#define DECLARE_DYNAMIC_DELEGATE_FiveParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name))
#define DECLARE_DYNAMIC_MULTICAST_DELEGATE_FiveParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name))
#define DECLARE_DELEGATE_RetVal_FiveParams(ReturnValueType, DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, ReturnValueType(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type))
#define DECLARE_DYNAMIC_DELEGATE_RetVal_FiveParams(ReturnValueType, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, ReturnValueType(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name))

#define DECLARE_DELEGATE_SixParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type))
#define DECLARE_MULTICAST_DELEGATE_SixParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type) UE_PRIVATE_DECLARE_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type))
#define DECLARE_TS_MULTICAST_DELEGATE_SixParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type) UE_PRIVATE_DECLARE_TS_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type))
#define DECLARE_EVENT_SixParams(OwningType, EventName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type) UE_PRIVATE_DECLARE_EVENT(OwningType, EventName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type))
#define DECLARE_DYNAMIC_DELEGATE_SixParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name, Param6Type Param6Name))
#define DECLARE_DYNAMIC_MULTICAST_DELEGATE_SixParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name, Param6Type Param6Name))
#define DECLARE_DELEGATE_RetVal_SixParams(ReturnValueType, DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, ReturnValueType(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type))
#define DECLARE_DYNAMIC_DELEGATE_RetVal_SixParams(ReturnValueType, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, ReturnValueType(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name, Param6Type Param6Name))

#define DECLARE_DELEGATE_SevenParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type))
#define DECLARE_MULTICAST_DELEGATE_SevenParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type) UE_PRIVATE_DECLARE_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type))
#define DECLARE_TS_MULTICAST_DELEGATE_SevenParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type) UE_PRIVATE_DECLARE_TS_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type))
#define DECLARE_EVENT_SevenParams(OwningType, EventName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type) UE_PRIVATE_DECLARE_EVENT(OwningType, EventName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type))
#define DECLARE_DYNAMIC_DELEGATE_SevenParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name, Param7Type, Param7Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name, Param6Type Param6Name, Param7Type Param7Name))
#define DECLARE_DYNAMIC_MULTICAST_DELEGATE_SevenParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name, Param7Type, Param7Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name, Param6Type Param6Name, Param7Type Param7Name))
#define DECLARE_DELEGATE_RetVal_SevenParams(ReturnValueType, DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, ReturnValueType(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type))
#define DECLARE_DYNAMIC_DELEGATE_RetVal_SevenParams(ReturnValueType, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name, Param7Type, Param7Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, ReturnValueType(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name, Param6Type Param6Name, Param7Type Param7Name))

#define DECLARE_DELEGATE_EightParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type))
#define DECLARE_MULTICAST_DELEGATE_EightParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type) UE_PRIVATE_DECLARE_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type))
#define DECLARE_TS_MULTICAST_DELEGATE_EightParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type) UE_PRIVATE_DECLARE_TS_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type))
#define DECLARE_EVENT_EightParams(OwningType, EventName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type) UE_PRIVATE_DECLARE_EVENT(OwningType, EventName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type))
#define DECLARE_DYNAMIC_DELEGATE_EightParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name, Param7Type, Param7Name, Param8Type, Param8Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param5Name, Param5Type Param5Name, Param6Type Param6Name, Param7Type Param7Name, Param8Type Param8Name))
#define DECLARE_DYNAMIC_MULTICAST_DELEGATE_EightParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name, Param7Type, Param7Name, Param8Type, Param8Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name, Param6Type Param6Name, Param7Type Param7Name, Param8Type Param8Name))
#define DECLARE_DELEGATE_RetVal_EightParams(ReturnValueType, DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, ReturnValueType(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type))
#define DECLARE_DYNAMIC_DELEGATE_RetVal_EightParams(ReturnValueType, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name, Param7Type, Param7Name, Param8Type, Param8Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, ReturnValueType(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name, Param6Type Param6Name, Param7Type Param7Name, Param8Type Param8Name))

#define DECLARE_DELEGATE_NineParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type, Param9Type) UE_PRIVATE_DECLARE_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type, Param9Type))
#define DECLARE_MULTICAST_DELEGATE_NineParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type, Param9Type) UE_PRIVATE_DECLARE_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type, Param9Type))
#define DECLARE_TS_MULTICAST_DELEGATE_NineParams(DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type, Param9Type) UE_PRIVATE_DECLARE_TS_MULTICAST_DELEGATE(DelegateName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type, Param9Type))
#define DECLARE_EVENT_NineParams(OwningType, EventName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type, Param9Type) UE_PRIVATE_DECLARE_EVENT(OwningType, EventName, void(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type, Param9Type))
#define DECLARE_DYNAMIC_DELEGATE_NineParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name, Param7Type, Param7Name, Param8Type, Param8Name, Param9Type, Param9Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name, Param6Type Param6Name, Param7Type Param7Name, Param8Type Param8Name, Param9Type Param9Name))
#define DECLARE_DYNAMIC_MULTICAST_DELEGATE_NineParams(DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name, Param7Type, Param7Name, Param8Type, Param8Name, Param9Type, Param9Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_DELEGATE(DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name, Param6Type Param6Name, Param7Type Param7Name, Param8Type Param8Name, Param9Type Param9Name))
#define DECLARE_DELEGATE_RetVal_NineParams(ReturnValueType, DelegateName, Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type, Param9Type) UE_PRIVATE_DECLARE_DELEGATE( DelegateName, ReturnValueType(Param1Type, Param2Type, Param3Type, Param4Type, Param5Type, Param6Type, Param7Type, Param8Type, Param9Type))
#define DECLARE_DYNAMIC_DELEGATE_RetVal_NineParams(ReturnValueType, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name, Param7Type, Param7Name, Param8Type, Param8Name, Param9Type, Param9Name) UE_PRIVATE_DECLARE_DYNAMIC_DELEGATE(DelegateName, ReturnValueType(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name, Param6Type Param6Name, Param7Type Param7Name, Param8Type Param8Name, Param9Type Param9Name))
