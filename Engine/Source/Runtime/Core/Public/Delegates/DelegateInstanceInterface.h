// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_exports

#include "Misc/AssertionMacros.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/Tuple.h"

class IDelegateInstance;

template <typename FuncType, typename UserPolicy>
struct IBaseDelegateInstance;

template <typename>
class TDelegateBase;

struct FNotThreadSafeNotCheckedDelegateMode;
struct FThreadSafeDelegateMode;
struct FNotThreadSafeDelegateMode;

struct FPrivateDelegateAllocation;

// IWYU pragma: end_exports

template <typename RetType, typename... ArgTypes, typename UserPolicy>
struct IBaseDelegateInstance<RetType(ArgTypes...), UserPolicy> : public UserPolicy::FDelegateInstanceExtras
{
	/**
	 * Emplaces a copy of the delegate instance into the given target delegate.
	 * Target MUST be private since no locks are taken! (CreateCopy into a temporary, then move.)
	 */
	virtual void CreateCopy(const FPrivateDelegateAllocation& Target) const = 0;

	/**
	 * Execute the delegate.  If the function pointer is not valid, an error will occur.
	 */
	virtual RetType Execute(ArgTypes...) const = 0;

	/**
	 * Execute the delegate, but only if the function pointer is still valid
	 *
	 * @return  Returns true if the function was executed
	 */
	// NOTE: Currently only delegates with no return value support ExecuteIfSafe()
	virtual bool ExecuteIfSafe(ArgTypes...) const = 0;
};

// Temp workaround for deprecation warnings being emitted by this template with VS2022 17.9. Reported fixed in 17.10
PRAGMA_DISABLE_DEPRECATION_WARNINGS

template <bool bConst, typename Class, typename FuncType>
struct TMemFunPtrType;

template <typename Class, typename RetType, typename... ArgTypes>
struct TMemFunPtrType<false, Class, RetType(ArgTypes...)>
{
	typedef RetType (Class::* Type)(ArgTypes...);
};

template <typename Class, typename RetType, typename... ArgTypes>
struct TMemFunPtrType<true, Class, RetType(ArgTypes...)>
{
	typedef RetType (Class::* Type)(ArgTypes...) const;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS
// End temp workaround

template <typename FuncType>
struct TPayload;

template <typename Ret, typename... Types>
struct TPayload<Ret(Types...)>
{
	TTuple<Types..., Ret> Values;

	template <typename... ArgTypes>
	explicit TPayload(ArgTypes&&... Args)
		: Values(Forward<ArgTypes>(Args)..., Ret())
	{
	}

	Ret& GetResult()
	{
		return Values.template Get<sizeof...(Types)>();
	}
};

template <typename... Types>
struct TPayload<void(Types...)>
{
	TTuple<Types...> Values;

	template <typename... ArgTypes>
	explicit TPayload(ArgTypes&&... Args)
		: Values(Forward<ArgTypes>(Args)...)
	{
	}

	void GetResult()
	{
	}
};

template <typename T>
struct TPlacementNewer
{
	TPlacementNewer()
		: bConstructed(false)
	{
	}

	~TPlacementNewer()
	{
		if (bConstructed)
		{
			// We need a typedef here because VC won't compile the destructor call below if T itself has a member called T
			typedef T TPlacementNewerTTypeTypedef;

			((TPlacementNewerTTypeTypedef*)&Bytes)->TPlacementNewerTTypeTypedef::~TPlacementNewerTTypeTypedef();
		}
	}

	template <typename... ArgTypes>
	T* operator()(ArgTypes&&... Args)
	{
		check(!bConstructed);
		T* Result = new (&Bytes) T(Forward<ArgTypes>(Args)...);
		bConstructed = true;
		return Result;
	}

	T* operator->()
	{
		check(bConstructed);
		return (T*)&Bytes;
	}

private:
	TTypeCompatibleBytes<T> Bytes;
	bool                    bConstructed;
};
