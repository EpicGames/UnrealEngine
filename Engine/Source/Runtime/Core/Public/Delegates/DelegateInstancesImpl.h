// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	DelegateInstancesImpl.h: Inline implementation of delegate bindings.

	The types declared in this file are for internal use only. 
================================================================================*/

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "CoreTypes.h"
#include "Delegates/DelegateInstanceInterface.h"
#include "Delegates/DelegateInstancesImplFwd.h"
#include "Delegates/IDelegateInstance.h"
#include "Delegates/DelegateBase.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"

namespace UE::Delegates::Private
{
	constexpr bool IsUObjectPtr(const volatile UObjectBase*) { return true; }
	constexpr bool IsUObjectPtr(...)                         { return false; }
}

template <typename FuncType, typename UserPolicy, typename... VarTypes>
class TCommonDelegateInstanceState : protected IBaseDelegateInstance<FuncType, UserPolicy>
{
public:
	template <typename... InVarTypes>
	explicit TCommonDelegateInstanceState(InVarTypes&&... Vars)
		: Payload(Forward<InVarTypes>(Vars)...)
		, Handle (FDelegateHandle::GenerateNewHandle)
	{
	}

	FDelegateHandle GetHandle() const final
	{
		return Handle;
	}

protected:
	// Payload member variables (if any).
	TTuple<VarTypes...> Payload;

	// The handle of this delegate
	FDelegateHandle Handle;
};

/* Delegate binding types
 *****************************************************************************/

template <class UserClass, typename RetValType, typename... ParamTypes, typename UserPolicy, typename... VarTypes>
class TBaseUFunctionDelegateInstance<UserClass, RetValType(ParamTypes...), UserPolicy, VarTypes...> : public TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>
{
private:
	using Super            = TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>;
	using DelegateBaseType = typename UserPolicy::FDelegateExtras;

	static_assert(UE::Delegates::Private::IsUObjectPtr((UserClass*)nullptr), "You cannot use UFunction delegates with non UObject classes.");

public:
	template <typename... InVarTypes>
	explicit TBaseUFunctionDelegateInstance(UserClass* InUserObject, const FName& InFunctionName, InVarTypes&&... Vars)
		: Super        (Forward<InVarTypes>(Vars)...)
		, FunctionName (InFunctionName)
		, UserObjectPtr(InUserObject)
	{
		check(InFunctionName != NAME_None);

		if (InUserObject != nullptr)
		{
			CachedFunction = UserObjectPtr->FindFunctionChecked(InFunctionName);
		}
	}

	// IDelegateInstance interface

#if USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME

	FName TryGetBoundFunctionName() const final
	{
		return FunctionName;
	}

#endif

	UObject* GetUObject() const final
	{
		return (UObject*)UserObjectPtr.Get();
	}

	const void* GetObjectForTimerManager() const final
	{
		return UserObjectPtr.Get();
	}

	uint64 GetBoundProgramCounterForTimerManager() const final
	{
		return 0;
	}

	// Deprecated
	bool HasSameObject(FDelegateUserObjectConst InUserObject) const final
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (InUserObject.IsUObject())
		{
			return UserObjectPtr.HasSameObject(InUserObject.GetUObject());
		}
#endif
		return UserObjectPtr.Get() == InUserObject;
	}

	bool IsCompactable() const final
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return !UserObjectPtr.IsValid(true);
#else
		return !UserObjectPtr.Get(true);
#endif
	}

	bool IsSafeToExecute() const final
	{
		return UserObjectPtr.IsValid();
	}

public:

	// IBaseDelegateInstance interface

	void CreateCopy(const FPrivateDelegateAllocation& Target) const final
	{
		new (Target) TBaseUFunctionDelegateInstance(*this);
	}

	RetValType Execute(ParamTypes... Params) const final
	{
		using FParmsWithPayload = TPayload<RetValType(std::decay_t<ParamTypes>..., std::decay_t<VarTypes>...)>;

		checkSlow(IsSafeToExecute());

		TPlacementNewer<FParmsWithPayload> PayloadAndParams;
		this->Payload.ApplyAfter(PayloadAndParams, Forward<ParamTypes>(Params)...);
		UserObjectPtr->ProcessEvent(CachedFunction, &PayloadAndParams);
		return PayloadAndParams->GetResult();
	}

	bool ExecuteIfSafe(ParamTypes... Params) const final
	{
		TStrongObjectPtr<UserClass> PinnedObject = IsInGameThread() ? nullptr : this->UserObjectPtr.Pin();
		if (UserClass* ActualUserObject = this->UserObjectPtr.Get())
		{
			using FParmsWithPayload = TPayload<RetValType(std::decay_t<ParamTypes>..., std::decay_t<VarTypes>...)>;

			TPlacementNewer<FParmsWithPayload> PayloadAndParams;
			this->Payload.ApplyAfter(PayloadAndParams, Forward<ParamTypes>(Params)...);
			ActualUserObject->ProcessEvent(CachedFunction, &PayloadAndParams);
			return true;
		}

		return false;
	}

public:

	// Holds the cached UFunction to call.
	UFunction* CachedFunction;

	// Holds the name of the function to call.
	FName FunctionName;

	// The user object to call the function on.
	TWeakObjectPtr<UserClass> UserObjectPtr;
};


template <bool bConst, class UserClass, ESPMode SPMode, typename RetValType, typename... ParamTypes, typename UserPolicy, typename... VarTypes>
class TBaseSPMethodDelegateInstance<bConst, UserClass, SPMode, RetValType(ParamTypes...), UserPolicy, VarTypes...> : public TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>
{
private:
	using Super            = TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>;
	using DelegateBaseType = typename UserPolicy::FDelegateExtras;

public:
	using FMethodPtr = typename TMemFunPtrType<bConst, UserClass, RetValType(ParamTypes..., VarTypes...)>::Type;

	template <typename... InVarTypes>
	explicit TBaseSPMethodDelegateInstance(const TSharedPtr<UserClass, SPMode>& InUserObject, FMethodPtr InMethodPtr, InVarTypes&&... Vars)
		: Super     (Forward<InVarTypes>(Vars)...)
		, UserObject(InUserObject)
		, MethodPtr (InMethodPtr)
	{
		// NOTE: Shared pointer delegates are allowed to have a null incoming object pointer.  Weak pointers can expire,
		//       an it is possible for a copy of a delegate instance to end up with a null pointer.
		checkSlow(MethodPtr != nullptr);
	}

	// IDelegateInstance interface

#if USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME

	FName TryGetBoundFunctionName() const final
	{
		return NAME_None;
	}

#endif

	UObject* GetUObject() const final
	{
		return nullptr;
	}

	const void* GetObjectForTimerManager() const final
	{
		return UserObject.Pin().Get();
	}

	uint64 GetBoundProgramCounterForTimerManager() const final
	{
		return *((uint64*)&MethodPtr);
	}

	// Deprecated
	bool HasSameObject(FDelegateUserObjectConst InUserObject) const final
	{
		return UserObject.HasSameObject(InUserObject);
	}

	bool IsSafeToExecute() const final
	{
		return UserObject.IsValid();
	}

public:

	// IBaseDelegateInstance interface

	void CreateCopy(const FPrivateDelegateAllocation& Target) const final
	{
		new (Target) TBaseSPMethodDelegateInstance(*this);
	}

	RetValType Execute(ParamTypes... Params) const final
	{
		using MutableUserClass = std::remove_const_t<UserClass>;

		// Verify that the user object is still valid.  We only have a weak reference to it.
		TSharedPtr<UserClass, SPMode> SharedUserObject = UserObject.Pin();
		checkSlow(SharedUserObject.IsValid());

		// Safely remove const to work around a compiler issue with instantiating template permutations for 
		// overloaded functions that take a function pointer typedef as a member of a templated class.  In
		// all cases where this code is actually invoked, the UserClass will already be a const pointer.
		MutableUserClass* MutableUserObject = const_cast<MutableUserClass*>(SharedUserObject.Get());

		checkSlow(MethodPtr != nullptr);

		return this->Payload.ApplyAfter(MethodPtr, MutableUserObject, Forward<ParamTypes>(Params)...);
	}

	bool ExecuteIfSafe(ParamTypes... Params) const final
	{
		// Verify that the user object is still valid.  We only have a weak reference to it.
		if (TSharedPtr<UserClass, SPMode> SharedUserObject = this->UserObject.Pin())
		{
			using MutableUserClass = std::remove_const_t<UserClass>;

			// Safely remove const to work around a compiler issue with instantiating template permutations for 
			// overloaded functions that take a function pointer typedef as a member of a templated class.  In
			// all cases where this code is actually invoked, the UserClass will already be a const pointer.
			MutableUserClass* MutableUserObject = const_cast<MutableUserClass*>(SharedUserObject.Get());

			checkSlow(MethodPtr != nullptr);

			(void)this->Payload.ApplyAfter(MethodPtr, MutableUserObject, Forward<ParamTypes>(Params)...);

			return true;
		}

		return false;
	}

protected:

	// Weak reference to an instance of the user's class which contains a method we would like to call.
	TWeakPtr<UserClass, SPMode> UserObject;

	// C++ member function pointer.
	FMethodPtr MethodPtr;
};


template <ESPMode SPMode, typename RetValType, typename... ParamTypes, typename UserPolicy, typename FunctorType, typename... VarTypes>
class TBaseSPLambdaDelegateInstance<SPMode, RetValType(ParamTypes...), UserPolicy, FunctorType, VarTypes...> : public TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>
{
private:
	static_assert(std::is_same_v<FunctorType, std::remove_reference_t<FunctorType>>, "FunctorType cannot be a reference");

	using Super = TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>;
	using DelegateBaseType = typename UserPolicy::FDelegateExtras;

	using UserClass = const void;

public:
	template <typename InFunctorType, typename... InVarTypes>
	explicit TBaseSPLambdaDelegateInstance(const TWeakPtr<UserClass, SPMode>& InContextObject, InFunctorType&& InFunctor, InVarTypes&&... Vars)
		: Super(Forward<InVarTypes>(Vars)...)
		, ContextObject(InContextObject)
		, Functor(Forward<InFunctorType>(InFunctor))
	{
	}

	// IDelegateInstance interface

#if USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME

	FName TryGetBoundFunctionName() const final
	{
		return NAME_None;
	}

#endif

	UObject* GetUObject() const final
	{
		return nullptr;
	}

	const void* GetObjectForTimerManager() const final
	{
		return ContextObject.Pin().Get();
	}

	uint64 GetBoundProgramCounterForTimerManager() const final
	{
		return 0;
	}

	// Deprecated
	bool HasSameObject(FDelegateUserObjectConst InContextObject) const final
	{
		return ContextObject.Pin().Get() == InContextObject;
	}

	bool IsSafeToExecute() const final
	{
		return ContextObject.IsValid();
	}

public:

public:
	// IBaseDelegateInstance interface
	void CreateCopy(const FPrivateDelegateAllocation& Target) const final
	{
		new (Target) TBaseSPLambdaDelegateInstance(*this);
	}

	RetValType Execute(ParamTypes... Params) const final
	{
		return this->Payload.ApplyAfter(Functor, Forward<ParamTypes>(Params)...);
	}

	bool ExecuteIfSafe(ParamTypes... Params) const final
	{
		if (ContextObject.IsValid())
		{
			(void)this->Payload.ApplyAfter(Functor, Forward<ParamTypes>(Params)...);
			return true;
		}

		return false;
	}

private:

	// Weak reference to an instance of the user's class that controls the validity of the lambda.
	TWeakPtr<UserClass, SPMode> ContextObject;

	// We make this mutable to allow mutable lambdas to be bound and executed.
	mutable std::remove_const_t<FunctorType> Functor;
};


template <bool bConst, class UserClass, typename RetValType, typename... ParamTypes, typename UserPolicy, typename... VarTypes>
class TBaseRawMethodDelegateInstance<bConst, UserClass, RetValType(ParamTypes...), UserPolicy, VarTypes...> : public TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>
{
private:
	static_assert(!UE::Delegates::Private::IsUObjectPtr((UserClass*)nullptr), "You cannot use raw method delegates with UObjects.");

	using Super            = TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>;
	using DelegateBaseType = typename UserPolicy::FDelegateExtras;

public:
	using FMethodPtr = typename TMemFunPtrType<bConst, UserClass, RetValType(ParamTypes..., VarTypes...)>::Type;

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InUserObject An arbitrary object (templated) that hosts the member function.
	 * @param InMethodPtr C++ member function pointer for the method to bind.
	 */
	template <typename... InVarTypes>
	explicit TBaseRawMethodDelegateInstance(UserClass* InUserObject, FMethodPtr InMethodPtr, InVarTypes&&... Vars)
		: Super     (Forward<InVarTypes>(Vars)...)
		, UserObject(InUserObject)
		, MethodPtr (InMethodPtr)
	{
		// Non-expirable delegates must always have a non-null object pointer on creation (otherwise they could never execute.)
		check(InUserObject != nullptr && MethodPtr != nullptr);
	}

	// IDelegateInstance interface

#if USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME

	FName TryGetBoundFunctionName() const final
	{
		return NAME_None;
	}

#endif

	UObject* GetUObject() const final
	{
		return nullptr;
	}

	const void* GetObjectForTimerManager() const final
	{
		return UserObject;
	}

	uint64 GetBoundProgramCounterForTimerManager() const final
	{
		return *((uint64*)&MethodPtr);
	}

	// Deprecated
	bool HasSameObject(FDelegateUserObjectConst InUserObject) const final
	{
		return UserObject == InUserObject;
	}

	bool IsSafeToExecute() const final
	{
		// We never know whether or not it is safe to deference a C++ pointer, but we have to
		// trust the user in this case.  Prefer using a shared-pointer based delegate type instead!
		return true;
	}

public:

	// IBaseDelegateInstance interface

	void CreateCopy(const FPrivateDelegateAllocation& Target) const final
	{
		new (Target) TBaseRawMethodDelegateInstance(*this);
	}

	RetValType Execute(ParamTypes... Params) const final
	{
		using MutableUserClass = std::remove_const_t<UserClass>;

		// Safely remove const to work around a compiler issue with instantiating template permutations for 
		// overloaded functions that take a function pointer typedef as a member of a templated class.  In
		// all cases where this code is actually invoked, the UserClass will already be a const pointer.
		MutableUserClass* MutableUserObject = const_cast<MutableUserClass*>(UserObject);

		checkSlow(MethodPtr != nullptr);

		return this->Payload.ApplyAfter(MethodPtr, MutableUserObject, Forward<ParamTypes>(Params)...);
	}


	bool ExecuteIfSafe(ParamTypes... Params) const final
	{
		using MutableUserClass = std::remove_const_t<UserClass>;

		// Safely remove const to work around a compiler issue with instantiating template permutations for 
		// overloaded functions that take a function pointer typedef as a member of a templated class.  In
		// all cases where this code is actually invoked, the UserClass will already be a const pointer.
		MutableUserClass* MutableUserObject = const_cast<MutableUserClass*>(UserObject);

		checkSlow(MethodPtr != nullptr);

		(void)this->Payload.ApplyAfter(MethodPtr, MutableUserObject, Forward<ParamTypes>(Params)...);

		return true;
	}

protected:

	// Pointer to the user's class which contains a method we would like to call.
	UserClass* UserObject;

	// C++ member function pointer.
	FMethodPtr MethodPtr;
};


template <bool bConst, class UserClass, typename RetValType, typename... ParamTypes, typename UserPolicy, typename... VarTypes>
class TBaseUObjectMethodDelegateInstance<bConst, UserClass, RetValType(ParamTypes...), UserPolicy, VarTypes...> : public TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>
{
private:
	using Super            = TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>;
	using DelegateBaseType = typename UserPolicy::FDelegateExtras;

	static_assert(UE::Delegates::Private::IsUObjectPtr((UserClass*)nullptr), "You cannot use UObject method delegates with raw pointers.");

public:
	using FMethodPtr = typename TMemFunPtrType<bConst, UserClass, RetValType(ParamTypes..., VarTypes...)>::Type;

	template <typename... InVarTypes>
	explicit TBaseUObjectMethodDelegateInstance(UserClass* InUserObject, FMethodPtr InMethodPtr, InVarTypes&&... Vars)
		: Super     (Forward<InVarTypes>(Vars)...)
		, UserObject(InUserObject)
		, MethodPtr (InMethodPtr)
	{
		// NOTE: UObject delegates are allowed to have a null incoming object pointer.  UObject weak pointers can expire,
		//       an it is possible for a copy of a delegate instance to end up with a null pointer.
		checkSlow(MethodPtr != nullptr);
	}

	// IDelegateInstance interface

#if USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME

	FName TryGetBoundFunctionName() const final
	{
		return NAME_None;
	}

#endif

	UObject* GetUObject() const final
	{
		return (UObject*)UserObject.Get();
	}

	const void* GetObjectForTimerManager() const final
	{
		return UserObject.Get();
	}

	uint64 GetBoundProgramCounterForTimerManager() const final
	{
		return *((uint64*)&MethodPtr);
	}

	// Deprecated
	bool HasSameObject(FDelegateUserObjectConst InUserObject) const final
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (InUserObject.IsUObject())
		{
			return UserObject.HasSameObject(InUserObject.GetUObject());
		}
#endif
		return (UserObject.Get() == InUserObject);
	}

	bool IsCompactable() const final
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return !UserObject.IsValid(true);
#else
		return !UserObject.Get(true);
#endif
	}

	bool IsSafeToExecute() const final
	{
		return !!UserObject.Get();
	}

public:

	// IBaseDelegateInstance interface

	void CreateCopy(const FPrivateDelegateAllocation& Target) const final
	{
		new (Target) TBaseUObjectMethodDelegateInstance(*this);
	}

	RetValType Execute(ParamTypes... Params) const final
	{
		using MutableUserClass = std::remove_const_t<UserClass>;

		// Verify that the user object is still valid.  We only have a weak reference to it.
		checkSlow(UserObject.IsValid());

		// Safely remove const to work around a compiler issue with instantiating template permutations for 
		// overloaded functions that take a function pointer typedef as a member of a templated class.  In
		// all cases where this code is actually invoked, the UserClass will already be a const pointer.
		MutableUserClass* MutableUserObject = const_cast<MutableUserClass*>(UserObject.Get());

		checkSlow(MethodPtr != nullptr);

		return this->Payload.ApplyAfter(MethodPtr, MutableUserObject, Forward<ParamTypes>(Params)...);
	}

	bool ExecuteIfSafe(ParamTypes... Params) const final
	{
		TStrongObjectPtr<UserClass> PinnedObject = IsInGameThread() ? nullptr : this->UserObject.Pin();
		if (UserClass* ActualUserObject = this->UserObject.Get())
		{
			using MutableUserClass = std::remove_const_t<UserClass>;

			// Safely remove const to work around a compiler issue with instantiating template permutations for 
			// overloaded functions that take a function pointer typedef as a member of a templated class.  In
			// all cases where this code is actually invoked, the UserClass will already be a const pointer.
			MutableUserClass* MutableUserObject = const_cast<MutableUserClass*>(ActualUserObject);

			checkSlow(MethodPtr != nullptr);

			(void)this->Payload.ApplyAfter(MethodPtr, MutableUserObject, Forward<ParamTypes>(Params)...);

			return true;
		}
		return false;
	}

protected:

	// Pointer to the user's class which contains a method we would like to call.
	TWeakObjectPtr<UserClass> UserObject;

	// C++ member function pointer.
	FMethodPtr MethodPtr;
};


template <typename RetValType, typename... ParamTypes, typename UserPolicy, typename... VarTypes>
class TBaseStaticDelegateInstance<RetValType(ParamTypes...), UserPolicy, VarTypes...> : public TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>
{
private:
	using Super            = TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>;
	using DelegateBaseType = typename UserPolicy::FDelegateExtras;

public:
	using FFuncPtr = RetValType(*)(ParamTypes..., VarTypes...);

	template <typename... InVarTypes>
	explicit TBaseStaticDelegateInstance(FFuncPtr InStaticFuncPtr, InVarTypes&&... Vars)
		: Super        (Forward<InVarTypes>(Vars)...)
		, StaticFuncPtr(InStaticFuncPtr)
	{
		check(StaticFuncPtr != nullptr);
	}

	// IDelegateInstance interface

#if USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME

	FName TryGetBoundFunctionName() const final
	{
		return NAME_None;
	}

#endif

	UObject* GetUObject() const final
	{
		return nullptr;
	}

	const void* GetObjectForTimerManager() const final
	{
		return nullptr;
	}

	uint64 GetBoundProgramCounterForTimerManager() const final
	{
		return *((uint64*)&StaticFuncPtr);
	}

	// Deprecated
	bool HasSameObject(FDelegateUserObjectConst UserObject) const final
	{
		// Raw Delegates aren't bound to an object so they can never match
		return false;
	}

	bool IsSafeToExecute() const final
	{
		// Static functions are always safe to execute!
		return true;
	}

public:

	// IBaseDelegateInstance interface

	void CreateCopy(const FPrivateDelegateAllocation& Target) const final
	{
		new (Target) TBaseStaticDelegateInstance(*this);
	}

	RetValType Execute(ParamTypes... Params) const final
	{
		// Call the static function
		checkSlow(StaticFuncPtr != nullptr);

		return this->Payload.ApplyAfter(StaticFuncPtr, Forward<ParamTypes>(Params)...);
	}

	bool ExecuteIfSafe(ParamTypes... Params) const final
	{
		// Call the static function
		checkSlow(StaticFuncPtr != nullptr);

		(void)this->Payload.ApplyAfter(StaticFuncPtr, Forward<ParamTypes>(Params)...);

		return true;
	}

private:

	// C++ function pointer.
	FFuncPtr StaticFuncPtr;
};


template <typename RetValType, typename... ParamTypes, typename UserPolicy, typename FunctorType, typename... VarTypes>
class TBaseFunctorDelegateInstance<RetValType(ParamTypes...), UserPolicy, FunctorType, VarTypes...> : public TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>
{
private:
	static_assert(std::is_same_v<FunctorType, std::remove_reference_t<FunctorType>>, "FunctorType cannot be a reference");

	using Super            = TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>;
	using DelegateBaseType = typename UserPolicy::FDelegateExtras;

public:
	template <typename InFunctorType, typename... InVarTypes>
	explicit TBaseFunctorDelegateInstance(InFunctorType&& InFunctor, InVarTypes&&... Vars)
		: Super  (Forward<InVarTypes>(Vars)...)
		, Functor(Forward<InFunctorType>(InFunctor))
	{
	}

	// IDelegateInstance interface

#if USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME

	FName TryGetBoundFunctionName() const final
	{
		return NAME_None;
	}

#endif

	UObject* GetUObject() const final
	{
		return nullptr;
	}

	const void* GetObjectForTimerManager() const final
	{
		return nullptr;
	}

	uint64 GetBoundProgramCounterForTimerManager() const final
	{
		return 0;
	}

	// Deprecated
	bool HasSameObject(FDelegateUserObjectConst UserObject) const final
	{
		// Functor Delegates aren't bound to a user object so they can never match
		return false;
	}

	bool IsSafeToExecute() const final
	{
		// Functors are always considered safe to execute!
		return true;
	}

public:
	// IBaseDelegateInstance interface
	void CreateCopy(const FPrivateDelegateAllocation& Target) const final
	{
		new (Target) TBaseFunctorDelegateInstance(*this);
	}

	RetValType Execute(ParamTypes... Params) const final
	{
		return this->Payload.ApplyAfter(Functor, Forward<ParamTypes>(Params)...);
	}

	bool ExecuteIfSafe(ParamTypes... Params) const final
	{
		// Functors are always considered safe to execute!
		(void)this->Payload.ApplyAfter(Functor, Forward<ParamTypes>(Params)...);

		return true;
	}

private:
	// C++ functor
	// We make this mutable to allow mutable lambdas to be bound and executed.  We don't really want to
	// model the Functor as being a direct subobject of the delegate (which would maintain transivity of
	// const - because the binding doesn't affect the substitutability of a copied delegate.
	mutable std::remove_const_t<FunctorType> Functor;
};


template <typename RetValType, typename... ParamTypes, typename UserPolicy, typename FunctorType, typename... VarTypes>
class TWeakBaseFunctorDelegateInstance<RetValType(ParamTypes...), UserPolicy, FunctorType, VarTypes...> : public TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>
{
private:
	static_assert(std::is_same_v<FunctorType, std::remove_reference_t<FunctorType>>, "FunctorType cannot be a reference");

	using Super            = TCommonDelegateInstanceState<RetValType(ParamTypes...), UserPolicy, VarTypes...>;
	using DelegateBaseType = typename UserPolicy::FDelegateExtras;

	// A dependency on the template parameter is required here to stop early instantiation of types which are
	// defined in CoreUObject.
	using UserClass = std::conditional_t<sizeof...(ParamTypes) == 0, const UObject, const UObject>;

public:
	template <typename InFunctorType, typename... InVarTypes>
	explicit TWeakBaseFunctorDelegateInstance(UserClass* InContextObject, InFunctorType&& InFunctor, InVarTypes&&... Vars)
		: Super        (Forward<InVarTypes>(Vars)...)
		, ContextObject(InContextObject)
		, Functor      (Forward< InFunctorType>(InFunctor))
	{
	}

	// IDelegateInstance interface

#if USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME

	FName TryGetBoundFunctionName() const final
	{
		return NAME_None;
	}

#endif

	UObject* GetUObject() const final
	{
		return (UObject*)ContextObject.Get();
	}

	const void* GetObjectForTimerManager() const final
	{
		return ContextObject.Get();
	}

	uint64 GetBoundProgramCounterForTimerManager() const final
	{
		return 0;
	}

	// Deprecated
	bool HasSameObject(FDelegateUserObjectConst InContextObject) const final
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (InContextObject.IsUObject())
		{
			return ContextObject.HasSameObject(InContextObject.GetUObject());
		}
#endif
		return GetUObject() == InContextObject;
	}

	bool IsCompactable() const final
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return !ContextObject.IsValid(true);
#else
		return !ContextObject.Get(true);
#endif
	}

	bool IsSafeToExecute() const final
	{
		return ContextObject.IsValid();
	}

public:
	// IBaseDelegateInstance interface
	void CreateCopy(const FPrivateDelegateAllocation& Target) const final
	{
		new (Target) TWeakBaseFunctorDelegateInstance(*this);
	}

	RetValType Execute(ParamTypes... Params) const final
	{
		return this->Payload.ApplyAfter(Functor, Forward<ParamTypes>(Params)...);
	}

	bool ExecuteIfSafe(ParamTypes... Params) const final
	{
		TStrongObjectPtr<UserClass> PinnedObject = IsInGameThread() ? nullptr : ContextObject.Pin();
		if (ContextObject.IsValid())
		{
			(void)this->Payload.ApplyAfter(Functor, Forward<ParamTypes>(Params)...);
			return true;
		}

		return false;
	}

private:
	// Context object - the validity of this object controls the validity of the lambda
	TWeakObjectPtr<UserClass> ContextObject;

	// C++ functor
	// We make this mutable to allow mutable lambdas to be bound and executed.  We don't really want to
	// model the Functor as being a direct subobject of the delegate (which would maintain transivity of
	// const - because the binding doesn't affect the substitutability of a copied delegate.
	mutable std::remove_const_t<FunctorType> Functor;
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Templates/RemoveReference.h"
#endif
