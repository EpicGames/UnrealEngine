// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "CoreTypes.h"

enum class ESPMode : uint8;

/**
 * Implements a delegate binding for UFunctions.
 *
 * @params UserClass Must be an UObject derived class.
 */
template <class UserClass, typename FuncType, typename UserPolicy, typename... VarTypes>
class TBaseUFunctionDelegateInstance;

/**
 * Implements a delegate binding for shared pointer member functions.
 */
template <bool bConst, class UserClass, ESPMode SPMode, typename FuncType, typename UserPolicy, typename... VarTypes>
class TBaseSPMethodDelegateInstance;

/**
 * Implements a delegate binding for shared pointer functors, e.g. lambdas.
 */
template <ESPMode SPMode, typename FuncType, typename UserPolicy, typename FunctorType, typename... VarTypes>
class TBaseSPLambdaDelegateInstance;

/**
 * Implements a delegate binding for C++ member functions.
 */
template <bool bConst, class UserClass, typename FuncType, typename UserPolicy, typename... VarTypes>
class TBaseRawMethodDelegateInstance;

/**
 * Implements a delegate binding for UObject methods.
 */
template <bool bConst, class UserClass, typename FuncType, typename UserPolicy, typename... VarTypes>
class TBaseUObjectMethodDelegateInstance;

/**
 * Implements a delegate binding for regular C++ functions.
 */
template <typename FuncType, typename UserPolicy, typename... VarTypes>
class TBaseStaticDelegateInstance;

/**
 * Implements a delegate binding for C++ functors, e.g. lambdas.
 */
template <typename FuncType, typename UserPolicy, typename FunctorType, typename... VarTypes>
class TBaseFunctorDelegateInstance;

/**
 * Implements a weak object delegate binding for C++ functors, e.g. lambdas.
 */
template <typename FuncType, typename UserPolicy, typename FunctorType, typename... VarTypes>
class TWeakBaseFunctorDelegateInstance;
