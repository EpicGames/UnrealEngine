// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	#define USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME 1
#else
	#define USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME 0
#endif

