// Copyright Epic Games, Inc. All Rights Reserved.

// Only designed to be included directly by Delegate.h
#if !defined( __Delegate_h__ ) || !defined( FUNC_INCLUDING_INLINE_IMPL )
	#error "This inline header must only be included by Delegate.h"
#endif

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "CoreTypes.h"
#include "Concepts/Void.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/ScriptDelegates.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Templates/CopyQualifiersFromTo.h"
#include "Templates/Identity.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include <type_traits>

class FDelegateHandle;
class IDelegateInstance;
struct FWeakObjectPtr;
template <typename FuncType, typename UserPolicy> struct IBaseDelegateInstance;
template <typename T> struct TObjectPtr;

template <typename T>
T* ToRawPtr(const TObjectPtr<T>& Ptr);

template <typename To, typename From>
TCopyQualifiersFromTo_T<From, To>* Cast(From* Src);

template<typename UserPolicy> class TMulticastDelegateBase;

#if UE_DELEGATE_CHECK_LIFETIME && defined(UE_DYNAMIC_MODULE_NAME)
#include "Misc/SourceLocation.h"
#include "Misc/SourceLocationUtils.h"
#define UE_REGISTER_MULTICAST_INSTANCE(Handle) FMulticastDelegateTracker::RegisterDelegateInstance( \
	Handle,                                                                                          \
	UE_DYNAMIC_MODULE_NAME,                                                                          \
	UE::FSourceLocation::Current().GetFunctionName())
#else
#define UE_REGISTER_MULTICAST_INSTANCE(Handle)
#endif // UE_DELEGATE_CHECK_LIFETIME && defined(UE_DYNAMIC_MODULE_NAME)

/**
 * Delegate registration class.  A function can expose this base class to users to let them bind
 * against, but without letting them execute.  This gives us the benefit of C# events.
 *
 * Example:
 *
 * struct FInterestingThing
 * {
 * public:
 *     TDelegateRegistration<void()>& OnInterestingThing() const
 *     {
 *         return OnInterestingThingDelegate;
 *     }
 *
 * private:
 *     mutable TDelegate<void()> OnInterestingThingDelegate;
 * };
 *
 * void Func(const FInterestingThing& Thing)
 * {
 *     // Binding and unbinding are allowed
 *     Thing.OnInterestingThing().BindLambda([](){ RespondToInterestingThing(); });
 *     Thing.OnInterestingThing().Unbind();
 *
 *     // Execute is not
 *     Thing.OnInterestingThing().Execute(); // error: Execute is deleted
 * }
 */

template <typename DelegateSignature, typename UserPolicy = FDefaultDelegateUserPolicy>
class TDelegateRegistration;

template <typename InRetValType, typename... ParamTypes, typename UserPolicy>
class TDelegateRegistration<InRetValType(ParamTypes...), UserPolicy> : public UserPolicy::FDelegateExtras
{
private:
	using Super = typename UserPolicy::FDelegateExtras;

protected:
	using FuncType                      = InRetValType (ParamTypes...);
	using DelegateInstanceInterfaceType = IBaseDelegateInstance<FuncType, UserPolicy>;

private:
	static_assert(std::is_convertible_v<typename UserPolicy::FDelegateInstanceExtras*, IDelegateInstance*>, "UserPolicy::FDelegateInstanceExtras should publicly inherit IDelegateInstance");
	static_assert(std::is_convertible_v<typename UserPolicy::FMulticastDelegateExtras*, TMulticastDelegateBase<UserPolicy>*>, "UserPolicy::FMulticastDelegateExtras should publicly inherit TMulticastDelegateBase<UserPolicy>");

	template <typename, typename>
	friend class TDelegate;

	template <typename, typename>
	friend class TDelegateRegistration;

	template <typename>
	friend class TMulticastDelegateBase;

	template <typename, typename>
	friend class TMulticastDelegate;

	template <typename, typename>
	friend class TMulticastDelegateRegistration;

public:
	/** Type definition for return value type. */
	using RetValType = InRetValType;
	using TFuncType  = InRetValType(ParamTypes...);

protected:
	TDelegateRegistration() = default;
	TDelegateRegistration(TDelegateRegistration&&) = default;
	TDelegateRegistration(const TDelegateRegistration&) = default;
	TDelegateRegistration& operator=(TDelegateRegistration&&) = default;
	TDelegateRegistration& operator=(const TDelegateRegistration&) = default;
	~TDelegateRegistration() = default;

public:
	/**
	 * Binds a raw C++ pointer global function delegate
	 */
	template <typename... VarTypes>
	inline void BindStatic(typename TBaseStaticDelegateInstance<FuncType, UserPolicy, std::decay_t<VarTypes>...>::FFuncPtr InFunc, VarTypes&&... Vars)
	{
		new (TWriteLockedDelegateAllocation{*this}) TBaseStaticDelegateInstance<FuncType, UserPolicy, std::decay_t<VarTypes>...>(InFunc, Forward<VarTypes>(Vars)...);
	}
	
	/**
	 * Static: Binds a C++ lambda delegate
	 * technically this works for any functor types, but lambdas are the primary use case
	 */
	template<typename FunctorType, typename... VarTypes>
	inline void BindLambda(FunctorType&& InFunctor, VarTypes&&... Vars)
	{
		new (TWriteLockedDelegateAllocation{*this}) TBaseFunctorDelegateInstance<FuncType, UserPolicy, std::remove_reference_t<FunctorType>, std::decay_t<VarTypes>...>(Forward<FunctorType>(InFunctor), Forward<VarTypes>(Vars)...);
	}

	/**
	 * Static: Binds a weak shared pointer C++ lambda delegate
	 * technically this works for any functor types, but lambdas are the primary use case
	 */
	template<typename UserClass, ESPMode Mode, typename FunctorType, typename... VarTypes>
	inline void BindSPLambda(const TSharedRef<UserClass, Mode>& InUserObjectRef, FunctorType&& InFunctor, VarTypes&&... Vars)
	{
		new (TWriteLockedDelegateAllocation{*this}) TBaseSPLambdaDelegateInstance<Mode, FuncType, UserPolicy, std::remove_reference_t<FunctorType>, std::decay_t<VarTypes>...>(TWeakPtr<const void>(InUserObjectRef), Forward<FunctorType>(InFunctor), Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename FunctorType, typename... VarTypes>
	inline void BindSPLambda(const UserClass* InUserObject, FunctorType&& InFunctor, VarTypes&&... Vars)
	{
		new (TWriteLockedDelegateAllocation{*this}) TBaseSPLambdaDelegateInstance<decltype(InUserObject->AsShared())::Mode, FuncType, UserPolicy, std::remove_reference_t<FunctorType>, std::decay_t<VarTypes>...>(TWeakPtr<const void>(InUserObject->AsWeak()), Forward<FunctorType>(InFunctor), Forward<VarTypes>(Vars)...);
	}

	/**
	 * Static: Binds a weak object C++ lambda delegate
	 * technically this works for any functor types, but lambdas are the primary use case
	 */
	template<typename FunctorType, typename... VarTypes>
	inline void BindWeakLambda(const UObject* InUserObject, FunctorType&& InFunctor, VarTypes&&... Vars)
	{
		new (TWriteLockedDelegateAllocation{*this}) TWeakBaseFunctorDelegateInstance<FuncType, UserPolicy, std::remove_reference_t<FunctorType>, std::decay_t<VarTypes>...>(InUserObject, Forward<FunctorType>(InFunctor), Forward<VarTypes>(Vars)...);
	}

	/**
	 * Binds a raw C++ pointer delegate.
	 *
	 * Raw pointer doesn't use any sort of reference, so may be unsafe to call if the object was
	 * deleted out from underneath your delegate. Be careful when calling Execute()!
	 */
	template <typename UserClass, typename... VarTypes>
	inline void BindRaw(UserClass* InUserObject, typename TMemFunPtrType<false, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		new (TWriteLockedDelegateAllocation{*this}) TBaseRawMethodDelegateInstance<false, UserClass, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObject, InFunc, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	inline void BindRaw(const UserClass* InUserObject, typename TMemFunPtrType<true, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		new (TWriteLockedDelegateAllocation{*this}) TBaseRawMethodDelegateInstance<true, const UserClass, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObject, InFunc, Forward<VarTypes>(Vars)...);
	}

	/**
	 * Binds a shared pointer-based member function delegate.  Shared pointer delegates keep a weak reference to your object.  You can use ExecuteIfBound() to call them.
	 */
	template <typename UserClass, ESPMode Mode, typename... VarTypes>
	inline void BindSP(const TSharedRef<UserClass, Mode>& InUserObjectRef, typename TMemFunPtrType<false, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		new (TWriteLockedDelegateAllocation{*this}) TBaseSPMethodDelegateInstance<false, UserClass, Mode, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObjectRef, InFunc, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, ESPMode Mode, typename... VarTypes>
	inline void BindSP(const TSharedRef<UserClass, Mode>& InUserObjectRef, typename TMemFunPtrType<true, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		new (TWriteLockedDelegateAllocation{*this}) TBaseSPMethodDelegateInstance<true, const UserClass, Mode, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObjectRef, InFunc, Forward<VarTypes>(Vars)...);
	}

	/**
	 * Binds a shared pointer-based member function delegate.
	 *
	 * Shared pointer delegates keep a weak reference to your object.
	 * You can use ExecuteIfBound() to call them.
	 */
	template <typename UserClass, typename... VarTypes>
	inline void BindSP(UserClass* InUserObject, typename TMemFunPtrType<false, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		new (TWriteLockedDelegateAllocation{*this}) TBaseSPMethodDelegateInstance<false, UserClass, decltype(InUserObject->AsShared())::Mode, FuncType, UserPolicy, std::decay_t<VarTypes>...>(StaticCastSharedRef<UserClass>(InUserObject->AsShared()), InFunc, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	inline void BindSP(const UserClass* InUserObject, typename TMemFunPtrType<true, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		new (TWriteLockedDelegateAllocation{*this}) TBaseSPMethodDelegateInstance<true, const UserClass, decltype(InUserObject->AsShared())::Mode, FuncType, UserPolicy, std::decay_t<VarTypes>...>(StaticCastSharedRef<const UserClass>(InUserObject->AsShared()), InFunc, Forward<VarTypes>(Vars)...);
	}

	/**
	 * Binds a shared pointer-based (thread-safe) member function delegate.
	 *
	 * Shared pointer delegates keep a weak reference to your object.
	 * You can use ExecuteIfBound() to call them.
	 *
	 * Note: This function is redundant, but is retained for backwards compatibility.  BindSP() works in both thread-safe and not-thread-safe modes and should be preferred.
	 */
	template <typename UserClass, typename... VarTypes>
	inline void BindThreadSafeSP(const TSharedRef<UserClass, ESPMode::ThreadSafe>& InUserObjectRef, typename TMemFunPtrType<false, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		new (TWriteLockedDelegateAllocation{*this}) TBaseSPMethodDelegateInstance<false, UserClass, ESPMode::ThreadSafe, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObjectRef, InFunc, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	inline void BindThreadSafeSP(const TSharedRef<UserClass, ESPMode::ThreadSafe>& InUserObjectRef, typename TMemFunPtrType<true, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		new (TWriteLockedDelegateAllocation{*this}) TBaseSPMethodDelegateInstance<true, const UserClass, ESPMode::ThreadSafe, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObjectRef, InFunc, Forward<VarTypes>(Vars)...);
	}

	/**
	 * Binds a shared pointer-based (thread-safe) member function delegate.
	 *
	 * Shared pointer delegates keep a weak reference to your object.
	 * You can use ExecuteIfBound() to call them.
	 *
	 * Note: This function is redundant, but is retained for backwards compatibility.  BindSP() works in both thread-safe and not-thread-safe modes and should be preferred.
	 */
	template <typename UserClass, typename... VarTypes>
	inline void BindThreadSafeSP(UserClass* InUserObject, typename TMemFunPtrType<false, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		new (TWriteLockedDelegateAllocation{*this}) TBaseSPMethodDelegateInstance<false, UserClass, ESPMode::ThreadSafe, FuncType, UserPolicy, std::decay_t<VarTypes>...>(StaticCastSharedRef<UserClass>(InUserObject->AsShared()), InFunc, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	inline void BindThreadSafeSP(const UserClass* InUserObject, typename TMemFunPtrType<true, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		new (TWriteLockedDelegateAllocation{*this}) TBaseSPMethodDelegateInstance<true, const UserClass, ESPMode::ThreadSafe, FuncType, UserPolicy, std::decay_t<VarTypes>...>(StaticCastSharedRef<const UserClass>(InUserObject->AsShared()), InFunc, Forward<VarTypes>(Vars)...);
	}

	/**
	 * Binds a UFunction-based member function delegate.
	 *
	 * UFunction delegates keep a weak reference to your object.
	 * You can use ExecuteIfBound() to call them.
	 */
	template <typename UObjectTemplate, typename... VarTypes>
	inline void BindUFunction(UObjectTemplate* InUserObject, const FName& InFunctionName, VarTypes&&... Vars)
	{
		new (TWriteLockedDelegateAllocation{*this}) TBaseUFunctionDelegateInstance<UObjectTemplate, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UObjectTemplate, typename... VarTypes>
	inline void BindUFunction(TObjectPtr<UObjectTemplate> InUserObject, const FName& InFunctionName, VarTypes&&... Vars)
	{
		new (TWriteLockedDelegateAllocation{*this}) TBaseUFunctionDelegateInstance<UObjectTemplate, FuncType, UserPolicy, std::decay_t<VarTypes>...>(ToRawPtr(InUserObject), InFunctionName, Forward<VarTypes>(Vars)...);
	}

	/**
	 * Binds a UObject-based member function delegate.
	 *
	 * UObject delegates keep a weak reference to your object.
	 * You can use ExecuteIfBound() to call them.
	 */
	template <typename UserClass, typename... VarTypes>
	inline void BindUObject(UserClass* InUserObject, typename TMemFunPtrType<false, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		new (TWriteLockedDelegateAllocation{*this}) TBaseUObjectMethodDelegateInstance<false, UserClass, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObject, InFunc, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	inline void BindUObject(const UserClass* InUserObject, typename TMemFunPtrType<true, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		new (TWriteLockedDelegateAllocation{*this}) TBaseUObjectMethodDelegateInstance<true, const UserClass, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObject, InFunc, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	inline void BindUObject(TObjectPtr<UserClass> InUserObject, typename TMemFunPtrType<false, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		new (TWriteLockedDelegateAllocation{*this}) TBaseUObjectMethodDelegateInstance<false, UserClass, FuncType, UserPolicy, std::decay_t<VarTypes>...>(ToRawPtr(InUserObject), InFunc, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	inline void BindUObject(TObjectPtr<UserClass> InUserObject, typename TMemFunPtrType<true, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		new (TWriteLockedDelegateAllocation{*this}) TBaseUObjectMethodDelegateInstance<true, const UserClass, FuncType, UserPolicy, std::decay_t<VarTypes>...>(ToRawPtr(InUserObject), InFunc, Forward<VarTypes>(Vars)...);
	}

	// Executing via a delegate registration reference is not allowed
	RetValType Execute       (ParamTypes... Params) const = delete;
	bool       ExecuteIfBound(ParamTypes... Params) const = delete;
};

/**
 * Unicast delegate template class.
 */
template <typename DelegateSignature, typename UserPolicy = FDefaultDelegateUserPolicy>
class TDelegate
{
	static_assert(sizeof(UserPolicy) == 0, "Expected a function signature for the delegate template parameter");
};

template <typename InRetValType, typename... ParamTypes, typename UserPolicy>
class TDelegate<InRetValType(ParamTypes...), UserPolicy> : public TDelegateRegistration<InRetValType(ParamTypes...), UserPolicy>
{
private:
	using Super                         = TDelegateRegistration<InRetValType(ParamTypes...), UserPolicy>;
	using DelegateInstanceInterfaceType = typename Super::DelegateInstanceInterfaceType;
	using FuncType                      = typename Super::FuncType;

	// Make sure FDelegateBase's protected functions are not accidentally exposed through the TDelegate API
	using typename Super::FReadAccessScope;
	using Super::GetReadAccessScope;
	using typename Super::FWriteAccessScope;
	using Super::GetWriteAccessScope;

	template <typename>
	friend class TMulticastDelegateBase;

public:
	using RegistrationType = Super;

	/** Type definition for return value type. */
	using RetValType = InRetValType;
	using TFuncType  = InRetValType(ParamTypes...);

	/* Helper typedefs for getting a member function pointer type for the delegate with a given payload */
	template <typename... VarTypes>                     using TFuncPtr        = RetValType(*)(ParamTypes..., VarTypes...);
	template <typename UserClass, typename... VarTypes> using TMethodPtr      = typename TMemFunPtrType<false, UserClass, RetValType(ParamTypes..., VarTypes...)>::Type;
	template <typename UserClass, typename... VarTypes> using TConstMethodPtr = typename TMemFunPtrType<true,  UserClass, RetValType(ParamTypes..., VarTypes...)>::Type;

	TDelegate() = default;

	inline TDelegate(TYPE_OF_NULLPTR)
	{
	}

	inline TDelegate(const TDelegate& Other)
	{
		CopyFrom(Other);
	}

	TDelegate& operator=(const TDelegate& Other)
	{
		CopyFrom(Other);
		return *this;
	}

	TDelegate(TDelegate&& Other) = default;
	TDelegate& operator=(TDelegate&& Other) = default;
	~TDelegate() = default;

	/**
	 * Static: Creates a raw C++ pointer global function delegate
	 */
	template <typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateStatic(TIdentity_T<RetValType (*)(ParamTypes..., std::decay_t<VarTypes>...)> InFunc, VarTypes&&... Vars)
	{
		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseStaticDelegateInstance<FuncType, UserPolicy, std::decay_t<VarTypes>...>(InFunc, Forward<VarTypes>(Vars)...);
		return Result;
	}

	/**
	 * Static: Creates a C++ lambda delegate
	 * technically this works for any functor types, but lambdas are the primary use case
	 */
	template<typename FunctorType, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateLambda(FunctorType&& InFunctor, VarTypes&&... Vars)
	{
		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseFunctorDelegateInstance<FuncType, UserPolicy, std::remove_reference_t<FunctorType>, std::decay_t<VarTypes>...>(Forward<FunctorType>(InFunctor), Forward<VarTypes>(Vars)...);
		return Result;
	}

	/**
	 * Static: Creates a weak shared pointer C++ lambda delegate
	 * technically this works for any functor types, but lambdas are the primary use case
	 */
	template<typename UserClass, ESPMode Mode, typename FunctorType, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateSPLambda(const TSharedRef<UserClass, Mode>& InUserObjectRef, FunctorType&& InFunctor, VarTypes&&... Vars)
	{
		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseSPLambdaDelegateInstance<Mode, FuncType, UserPolicy, std::remove_reference_t<FunctorType>, std::decay_t<VarTypes>...>(InUserObjectRef, Forward<FunctorType>(InFunctor), Forward<VarTypes>(Vars)...);
		return Result;
	}
	template <typename UserClass, typename FunctorType, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateSPLambda(UserClass* InUserObject, FunctorType&& InFunctor, VarTypes&&... Vars)
	{
		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseSPLambdaDelegateInstance<decltype(InUserObject->AsWeak())::Mode, FuncType, UserPolicy, std::remove_reference_t<FunctorType>, std::decay_t<VarTypes>...>(TWeakPtr<const void>(InUserObject->AsWeak()), Forward<FunctorType>(InFunctor), Forward<VarTypes>(Vars)...);
		return Result;
	}

	/**
	 * Static: Creates a weak object C++ lambda delegate
	 * technically this works for any functor types, but lambdas are the primary use case
	 */
	template<typename FunctorType, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateWeakLambda(const UObject* InUserObject, FunctorType&& InFunctor, VarTypes&&... Vars)
	{
		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TWeakBaseFunctorDelegateInstance<FuncType, UserPolicy, std::remove_reference_t<FunctorType>, std::decay_t<VarTypes>...>(InUserObject, Forward<FunctorType>(InFunctor), Forward<VarTypes>(Vars)...);
		return Result;
	}

	/**
	 * Static: Creates a raw C++ pointer member function delegate.
	 *
	 * Raw pointer doesn't use any sort of reference, so may be unsafe to call if the object was
	 * deleted out from underneath your delegate. Be careful when calling Execute()!
	 */
	template <typename UserClass, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateRaw(UserClass* InUserObject, typename TMemFunPtrType<false, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseRawMethodDelegateInstance<false, UserClass, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObject, InFunc, Forward<VarTypes>(Vars)...);
		return Result;
	}
	template <typename UserClass, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateRaw(const UserClass* InUserObject, typename TMemFunPtrType<true, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseRawMethodDelegateInstance<true, const UserClass, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObject, InFunc, Forward<VarTypes>(Vars)...);
		return Result;
	}

	/**
	 * Static: Creates a shared pointer-based member function delegate.
	 *
	 * Shared pointer delegates keep a weak reference to your object.
	 * You can use ExecuteIfBound() to call them.
	 */
	template <typename UserClass, ESPMode Mode, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateSP(const TSharedRef<UserClass, Mode>& InUserObjectRef, typename TMemFunPtrType<false, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseSPMethodDelegateInstance<false, UserClass, Mode, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObjectRef, InFunc, Forward<VarTypes>(Vars)...);
		return Result;
	}
	template <typename UserClass, ESPMode Mode, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateSP(const TSharedRef<UserClass, Mode>& InUserObjectRef, typename TMemFunPtrType<true, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseSPMethodDelegateInstance<true, const UserClass, Mode, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObjectRef, InFunc, Forward<VarTypes>(Vars)...);
		return Result;
	}

	/**
	 * Static: Creates a shared pointer-based member function delegate.
	 *
	 * Shared pointer delegates keep a weak reference to your object.
	 * You can use ExecuteIfBound() to call them.
	 */
	template <typename UserClass, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateSP(UserClass* InUserObject, typename TMemFunPtrType<false, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseSPMethodDelegateInstance<false, UserClass, decltype(InUserObject->AsShared())::Mode, FuncType, UserPolicy, std::decay_t<VarTypes>...>(StaticCastSharedRef<UserClass>(InUserObject->AsShared()), InFunc, Forward<VarTypes>(Vars)...);
		return Result;
	}
	template <typename UserClass, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateSP(const UserClass* InUserObject, typename TMemFunPtrType<true, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseSPMethodDelegateInstance<true, const UserClass, decltype(InUserObject->AsShared())::Mode, FuncType, UserPolicy, std::decay_t<VarTypes>...>(StaticCastSharedRef<const UserClass>(InUserObject->AsShared()), InFunc, Forward<VarTypes>(Vars)...);
		return Result;
	}

	/**
	 * Static: Creates a shared pointer-based (thread-safe) member function delegate.
	 *
	 * Shared pointer delegates keep a weak reference to your object.
	 * You can use ExecuteIfBound() to call them.
	 *
	 * Note: This function is redundant, but is retained for backwards compatibility.  CreateSP() works in both thread-safe and not-thread-safe modes and should be preferred.
	 */
	template <typename UserClass, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateThreadSafeSP(const TSharedRef<UserClass, ESPMode::ThreadSafe>& InUserObjectRef, typename TMemFunPtrType<false, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseSPMethodDelegateInstance<false, UserClass, ESPMode::ThreadSafe, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObjectRef, InFunc, Forward<VarTypes>(Vars)...);
		return Result;
	}
	template <typename UserClass, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateThreadSafeSP(const TSharedRef<UserClass, ESPMode::ThreadSafe>& InUserObjectRef, typename TMemFunPtrType<true, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseSPMethodDelegateInstance<true, const UserClass, ESPMode::ThreadSafe, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObjectRef, InFunc, Forward<VarTypes>(Vars)...);
		return Result;
	}

	/**
	 * Static: Creates a shared pointer-based (thread-safe) member function delegate.
	 *
	 * Shared pointer delegates keep a weak reference to your object.
	 * You can use ExecuteIfBound() to call them.
	 *
	 * Note: This function is redundant, but is retained for backwards compatibility.  CreateSP() works in both thread-safe and not-thread-safe modes and should be preferred.
	 */
	template <typename UserClass, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateThreadSafeSP(UserClass* InUserObject, typename TMemFunPtrType<false, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseSPMethodDelegateInstance<false, UserClass, ESPMode::ThreadSafe, FuncType, UserPolicy, std::decay_t<VarTypes>...>(StaticCastSharedRef<UserClass>(InUserObject->AsShared()), InFunc, Forward<VarTypes>(Vars)...);
		return Result;
	}
	template <typename UserClass, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateThreadSafeSP(const UserClass* InUserObject, typename TMemFunPtrType<true, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseSPMethodDelegateInstance<true, const UserClass, ESPMode::ThreadSafe, FuncType, UserPolicy, std::decay_t<VarTypes>...>(StaticCastSharedRef<const UserClass>(InUserObject->AsShared()), InFunc, Forward<VarTypes>(Vars)...);
		return Result;
	}

	/**
	 * Static: Creates a UFunction-based member function delegate.
	 *
	 * UFunction delegates keep a weak reference to your object.
	 * You can use ExecuteIfBound() to call them.
	 */
	template <typename UObjectTemplate, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateUFunction(UObjectTemplate* InUserObject, const FName& InFunctionName, VarTypes&&... Vars)
	{
		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseUFunctionDelegateInstance<UObjectTemplate, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);
		return Result;
	}
	template <typename UObjectTemplate, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateUFunction(TObjectPtr<UObjectTemplate> InUserObject, const FName& InFunctionName, VarTypes&&... Vars)
	{
		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseUFunctionDelegateInstance<UObjectTemplate, FuncType, UserPolicy, std::decay_t<VarTypes>...>(ToRawPtr(InUserObject), InFunctionName, Forward<VarTypes>(Vars)...);
		return Result;
	}

	/**
	 * Static: Creates a UObject-based member function delegate.
	 *
	 * UObject delegates keep a weak reference to your object.
	 * You can use ExecuteIfBound() to call them.
	 */
	template <typename UserClass, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateUObject(UserClass* InUserObject, typename TMemFunPtrType<false, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseUObjectMethodDelegateInstance<false, UserClass, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObject, InFunc, Forward<VarTypes>(Vars)...);
		return Result;
	}
	template <typename UserClass, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateUObject(const UserClass* InUserObject, typename TMemFunPtrType<true, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseUObjectMethodDelegateInstance<true, const UserClass, FuncType, UserPolicy, std::decay_t<VarTypes>...>(InUserObject, InFunc, Forward<VarTypes>(Vars)...);
		return Result;
	}
	template <typename UserClass, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateUObject(TObjectPtr<UserClass> InUserObject, typename TMemFunPtrType<false, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseUObjectMethodDelegateInstance<false, UserClass, FuncType, UserPolicy, std::decay_t<VarTypes>...>(ToRawPtr(InUserObject), InFunc, Forward<VarTypes>(Vars)...);
		return Result;
	}
	template <typename UserClass, typename... VarTypes>
	[[nodiscard]] inline static TDelegate<RetValType(ParamTypes...), UserPolicy> CreateUObject(TObjectPtr<UserClass> InUserObject, typename TMemFunPtrType<true, UserClass, RetValType (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		TDelegate<RetValType(ParamTypes...), UserPolicy> Result;
		new (FPrivateDelegateAllocation{Result}) TBaseUObjectMethodDelegateInstance<true, const UserClass, FuncType, UserPolicy, std::decay_t<VarTypes>...>(ToRawPtr(InUserObject), InFunc, Forward<VarTypes>(Vars)...);
		return Result;
	}

public:
	/**
	 * Execute the delegate.
	 *
	 * If the function pointer is not valid, an error will occur. Check IsBound() before
	 * calling this method or use ExecuteIfBound() instead.
	 *
	 * @see ExecuteIfBound
	 */
	inline RetValType Execute(ParamTypes... Params) const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		const DelegateInstanceInterfaceType* LocalDelegateInstance = GetDelegateInstanceProtected();

		// If this assert goes off, Execute() was called before a function was bound to the delegate.
		// Consider using ExecuteIfBound() instead.
		checkSlow(LocalDelegateInstance != nullptr);

		return LocalDelegateInstance->Execute(Forward<ParamTypes>(Params)...);
	}

	/**
	 * Execute the delegate, but only if the function pointer is still valid
	 *
	 * @return  Returns true if the function was executed
	 */
	 // NOTE: Currently only delegates with no return value support ExecuteIfBound()
	inline bool ExecuteIfBound(ParamTypes... Params) const
		requires (std::is_void_v<RetValType>)
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		if (const DelegateInstanceInterfaceType* Ptr = GetDelegateInstanceProtected())
		{
			return Ptr->ExecuteIfSafe(Forward<ParamTypes>(Params)...);
		}

		return false;
	}

protected:
	/**
	 * Returns a pointer to the correctly-typed delegate instance.
	 */
	UE_FORCEINLINE_HINT DelegateInstanceInterfaceType* GetDelegateInstanceProtected()
	{
		return static_cast<DelegateInstanceInterfaceType*>(Super::GetDelegateInstanceProtected());
	}
	UE_FORCEINLINE_HINT const DelegateInstanceInterfaceType* GetDelegateInstanceProtected() const
	{
		return static_cast<const DelegateInstanceInterfaceType*>(Super::GetDelegateInstanceProtected());
	}

private:
	template<typename OtherUserPolicy>
	void CopyFrom(const TDelegate<FuncType, OtherUserPolicy>& Other)
	{
		if ((void*)&Other == (void*)this)
		{
			return;
		}

		// to not hold both delegates locked, make a local copy of `Other` and then move it into this instance
		TDelegate LocalCopy;

		{
			typename TDelegate<FuncType, OtherUserPolicy>::FReadAccessScope OtherReadScope = Other.GetReadAccessScope();

			// this down-cast is OK! allows for managing invocation list in the base class without requiring virtual functions
			using OtherDelegateInstanceInterfaceType = IBaseDelegateInstance<FuncType, OtherUserPolicy>;
			const OtherDelegateInstanceInterfaceType* OtherDelegateInstance = Other.GetDelegateInstanceProtected();

			if (OtherDelegateInstance != nullptr)
			{
				OtherDelegateInstance->CreateCopy(FPrivateDelegateAllocation{LocalCopy});
			}
		}

		// Actually move-assign from our temporary into this instance. This
		// is implemented in TDelegateBase and acquires a WriteAcessScope.
		*this = MoveTemp(LocalCopy);
	}

	// copying from delegates with different user policy
	template<typename OtherUserPolicy>
	explicit TDelegate(const TDelegate<FuncType, OtherUserPolicy>& Other)
	{
		CopyFrom(Other);
	}
};

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename DelegateSignature>
using TTSDelegateRegistration = TDelegateRegistration<DelegateSignature, FDefaultTSDelegateUserPolicy>;

template <typename DelegateSignature>
using TTSDelegate = TDelegate<DelegateSignature, FDefaultTSDelegateUserPolicy>;

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Multicast delegate registration class.  A function can expose this base class to users to let them bind
 * against, but without letting them broadcast.  This gives us the benefit of C# events.
 *
 * Example:
 *
 * struct FInterestingThing
 * {
 * public:
 *     TMulticastDelegateRegistration<void()>& OnInterestingThing() const
 *     {
 *         return OnInterestingThingDelegate;
 *     }
 *
 * private:
 *     mutable TMulticastDelegate<void()> OnInterestingThingDelegate;
 * };
 *
 * void Func(const FInterestingThing& Thing)
 * {
 *     // Registration and reregistration are allowed
 *     FDelegateHandle Handle = Thing.OnInterestingThing().Add([](){ RespondToInterestingThing(); });
 *     Thing.OnInterestingThing().Remove(Handle);
 *
 *     // Broadcast is not
 *     Thing.OnInterestingThing().Broadcast(); // error: Broadcast is deleted
 * }
 */
template <typename DelegateSignature, typename UserPolicy = FDefaultDelegateUserPolicy>
class TMulticastDelegateRegistration;

template <typename... ParamTypes, typename UserPolicy>
class TMulticastDelegateRegistration<void(ParamTypes...), UserPolicy> : public UserPolicy::FMulticastDelegateExtras
{
protected:
	using Super                         = typename UserPolicy::FMulticastDelegateExtras;
	using DelegateInstanceInterfaceType = IBaseDelegateInstance<void (ParamTypes...), UserPolicy>;

private:
	using InvocationListType = typename Super::InvocationListType;

public:
	/** Type definition for unicast delegate classes whose delegate instances are compatible with this delegate. */
	using FDelegate = TDelegate<void(ParamTypes...), UserPolicy>;

public:
	// Make sure TMulticastDelegateBase's public functions are publicly exposed through the TMulticastDelegateRegistration API
	using Super::Clear;
	using Super::IsBound;
	using Super::IsBoundToObject;
	using Super::RemoveAll;
	using Super::GetAllocatedSize;

private:
	// Make sure TMulticastDelegateBase's protected functions are not accidentally exposed through the TMulticastDelegateRegistration API
	using Super::AddDelegateInstance;
	using Super::RemoveDelegateInstance;

protected:
	TMulticastDelegateRegistration() = default;
	TMulticastDelegateRegistration(TMulticastDelegateRegistration&&) = default;
	TMulticastDelegateRegistration(const TMulticastDelegateRegistration&) = default;
	TMulticastDelegateRegistration& operator=(TMulticastDelegateRegistration&&) = default;
	TMulticastDelegateRegistration& operator=(const TMulticastDelegateRegistration&) = default;
	~TMulticastDelegateRegistration() = default;

public:

	/**
	 * Adds a delegate instance to this multicast delegate's invocation list.
	 *
	 * @param Delegate The delegate to add.
	 */
	FDelegateHandle Add(FDelegate&& InNewDelegate)
	{
		FDelegateHandle Handle = Super::AddDelegateInstance(MoveTemp(InNewDelegate));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}

	/**
	 * Adds a delegate instance to this multicast delegate's invocation list.
	 *
	 * @param Delegate The delegate to add.
	 */
	FDelegateHandle Add(const FDelegate& InNewDelegate)
	{
		FDelegateHandle Handle = Super::AddDelegateInstance(CopyTemp(InNewDelegate));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}

	/**
	 * Adds a raw C++ pointer global function delegate
	 *
	 * @param	InFunc	Function pointer
	 */
	template <typename... VarTypes>
	inline FDelegateHandle AddStatic(typename TBaseStaticDelegateInstance<void (ParamTypes...), UserPolicy, std::decay_t<VarTypes>...>::FFuncPtr InFunc, VarTypes&&... Vars)
	{
		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateStatic(InFunc, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}

	/**
	 * Adds a C++ lambda delegate
	 * technically this works for any functor types, but lambdas are the primary use case
	 *
	 * @param	InFunctor	Functor (e.g. Lambda)
	 */
	template<typename FunctorType, typename... VarTypes>
	inline FDelegateHandle AddLambda(FunctorType&& InFunctor, VarTypes&&... Vars)
	{
		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateLambda(Forward<FunctorType>(InFunctor), Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}

	/**
	 * Adds a weak shared pointer C++ lambda delegate
	 * technically this works for any functor types, but lambdas are the primary use case
	 *
	 * @param	InUserObjectRef	User object to bind to
	 * @param	InFunctor		Functor (e.g. Lambda)
	 */
	template <typename UserClass, typename FunctorType, typename... VarTypes>
	inline FDelegateHandle AddSPLambda(const UserClass* InUserObject, FunctorType&& InFunctor, VarTypes&&... Vars)
	{
		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateSPLambda(InUserObject, Forward<FunctorType>(InFunctor), Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}

	/**
	 * Adds a weak object C++ lambda delegate
	 * technically this works for any functor types, but lambdas are the primary use case
	 *
	 * @param	InUserObject	User object to bind to
	 * @param	InFunctor		Functor (e.g. Lambda)
	 */
	template<typename UserClass, typename FunctorType, typename... VarTypes>
	inline FDelegateHandle AddWeakLambda(UserClass* InUserObject, FunctorType&& InFunctor, VarTypes&&... Vars)
	{
		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateWeakLambda(InUserObject, Forward<FunctorType>(InFunctor), Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}

	/**
	 * Adds a raw C++ pointer delegate.
	 *
	 * Raw pointer doesn't use any sort of reference, so may be unsafe to call if the object was
	 * deleted out from underneath your delegate. Be careful when calling Execute()!
	 *
	 * @param	InUserObject	User object to bind to
	 * @param	InFunc			Class method function address
	 */
	template <typename UserClass, typename... VarTypes>
	inline FDelegateHandle AddRaw(UserClass* InUserObject, typename TMemFunPtrType<false, UserClass, void (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateRaw(InUserObject, InFunc, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}
	template <typename UserClass, typename... VarTypes>
	inline FDelegateHandle AddRaw(const UserClass* InUserObject, typename TMemFunPtrType<true, UserClass, void (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateRaw(InUserObject, InFunc, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}

	/**
	 * Adds a shared pointer-based member function delegate.
	 *
	 * Shared pointer delegates keep a weak reference to your object.
	 *
	 * @param	InUserObjectRef	User object to bind to
	 * @param	InFunc			Class method function address
	 */
	template <typename UserClass, ESPMode Mode, typename... VarTypes>
	inline FDelegateHandle AddSP(const TSharedRef<UserClass, Mode>& InUserObjectRef, typename TMemFunPtrType<false, UserClass, void (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateSP(InUserObjectRef, InFunc, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}
	template <typename UserClass, ESPMode Mode, typename... VarTypes>
	inline FDelegateHandle AddSP(const TSharedRef<UserClass, Mode>& InUserObjectRef, typename TMemFunPtrType<true, UserClass, void (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateSP(InUserObjectRef, InFunc, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}

	/**
	 * Adds a shared pointer-based member function delegate.
	 *
	 * Shared pointer delegates keep a weak reference to your object.
	 *
	 * @param	InUserObject	User object to bind to
	 * @param	InFunc			Class method function address
	 */
	template <typename UserClass, typename... VarTypes>
	inline FDelegateHandle AddSP(UserClass* InUserObject, typename TMemFunPtrType<false, UserClass, void (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateSP(InUserObject, InFunc, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}
	template <typename UserClass, typename... VarTypes>
	inline FDelegateHandle AddSP(const UserClass* InUserObject, typename TMemFunPtrType<true, UserClass, void (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateSP(InUserObject, InFunc, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}

	/**
	 * Adds a shared pointer-based (thread-safe) member function delegate.  Shared pointer delegates keep a weak reference to your object.
	 *
	 * @param	InUserObjectRef	User object to bind to
	 * @param	InFunc			Class method function address
	 *
	 * Note: This function is redundant, but is retained for backwards compatibility.  AddSP() works in both thread-safe and not-thread-safe modes and should be preferred.
	 */
	template <typename UserClass, typename... VarTypes>
	inline FDelegateHandle AddThreadSafeSP(const TSharedRef<UserClass, ESPMode::ThreadSafe>& InUserObjectRef, typename TMemFunPtrType<false, UserClass, void (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateThreadSafeSP(InUserObjectRef, InFunc, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}
	template <typename UserClass, typename... VarTypes>
	inline FDelegateHandle AddThreadSafeSP(const TSharedRef<UserClass, ESPMode::ThreadSafe>& InUserObjectRef, typename TMemFunPtrType<true, UserClass, void (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateThreadSafeSP(InUserObjectRef, InFunc, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}

	/**
	 * Adds a shared pointer-based (thread-safe) member function delegate.
	 *
	 * Shared pointer delegates keep a weak reference to your object.
	 *
	 * @param	InUserObject	User object to bind to
	 * @param	InFunc			Class method function address
	 *
	 * Note: This function is redundant, but is retained for backwards compatibility.  AddSP() works in both thread-safe and not-thread-safe modes and should be preferred.
	 */
	template <typename UserClass, typename... VarTypes>
	inline FDelegateHandle AddThreadSafeSP(UserClass* InUserObject, typename TMemFunPtrType<false, UserClass, void (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateThreadSafeSP(InUserObject, InFunc, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}
	template <typename UserClass, typename... VarTypes>
	inline FDelegateHandle AddThreadSafeSP(const UserClass* InUserObject, typename TMemFunPtrType<true, UserClass, void (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateThreadSafeSP(InUserObject, InFunc, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}

	/**
	 * Adds a UFunction-based member function delegate.
	 *
	 * UFunction delegates keep a weak reference to your object.
	 *
	 * @param	InUserObject	User object to bind to
	 * @param	InFunctionName			Class method function address
	 */
	template <typename UObjectTemplate, typename... VarTypes>
	inline FDelegateHandle AddUFunction(UObjectTemplate* InUserObject, const FName& InFunctionName, VarTypes&&... Vars)
	{
		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateUFunction(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}
	template <typename UObjectTemplate, typename... VarTypes>
	inline FDelegateHandle AddUFunction(TObjectPtr<UObjectTemplate> InUserObject, const FName& InFunctionName, VarTypes&&... Vars)
	{
		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateUFunction(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}

	/**
	 * Adds a UObject-based member function delegate.
	 *
	 * UObject delegates keep a weak reference to your object.
	 *
	 * @param	InUserObject	User object to bind to
	 * @param	InFunc			Class method function address
	 */
	template <typename UserClass, typename... VarTypes>
	inline FDelegateHandle AddUObject(UserClass* InUserObject, typename TMemFunPtrType<false, UserClass, void (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateUObject(InUserObject, InFunc, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}
	template <typename UserClass, typename... VarTypes>
	inline FDelegateHandle AddUObject(const UserClass* InUserObject, typename TMemFunPtrType<true, UserClass, void (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateUObject(InUserObject, InFunc, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}
	template <typename UserClass, typename... VarTypes>
	inline FDelegateHandle AddUObject(TObjectPtr<UserClass> InUserObject, typename TMemFunPtrType<false, UserClass, void (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		static_assert(!std::is_const_v<UserClass>, "Attempting to bind a delegate with a const object pointer and non-const member function.");

		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateUObject(InUserObject, InFunc, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}
	template <typename UserClass, typename... VarTypes>
	inline FDelegateHandle AddUObject(TObjectPtr<UserClass> InUserObject, typename TMemFunPtrType<true, UserClass, void (ParamTypes..., std::decay_t<VarTypes>...)>::Type InFunc, VarTypes&&... Vars)
	{
		FDelegateHandle Handle = Super::AddDelegateInstance(FDelegate::CreateUObject(InUserObject, InFunc, Forward<VarTypes>(Vars)...));
		UE_REGISTER_MULTICAST_INSTANCE(Handle);
		return Handle;
	}

public:

	/**
	 * Removes a delegate instance from this multi-cast delegate's invocation list (performance is O(N)).
	 *
	 * Note that the order of the delegate instances may not be preserved!
	 *
	 * @param Handle The handle of the delegate instance to remove.
	 * @return  true if the delegate was successfully removed.
	 */
	bool Remove( FDelegateHandle Handle )
	{
		bool bResult = false;
		if (Handle.IsValid())
		{
			bResult = RemoveDelegateInstance(Handle);
		}
		return bResult;
	}

	// Broadcasting via a delegate registration reference is not allowed
	void Broadcast(ParamTypes... Params) = delete;
};

/**
 * Multicast delegate template base class, used for both normal and event multicast delegates.
 *
 * This class implements the functionality of multicast delegates. It is templated to the function signature
 * that it is compatible with. Use the various DECLARE_MULTICAST_DELEGATE and DECLARE_EVENT macros to create
 * actual delegate types.
 *
 * Multicast delegates offer no guarantees for the calling order of bound functions. As bindings get added
 * and removed over time, the calling order may change. Only bindings without return values are supported.
 */
template <typename DelegateSignature, typename UserPolicy = FDefaultDelegateUserPolicy>
class TMulticastDelegate
{
	static_assert(sizeof(DelegateSignature) == 0, "Expected a function signature for the delegate template parameter");
};

template <typename RetValType, typename... ParamTypes, typename UserPolicy>
class TMulticastDelegate<RetValType(ParamTypes...), UserPolicy>
{
	static_assert(sizeof(RetValType) == 0, "The return type of a multicast delegate must be void");
};

template <typename... ParamTypes, typename UserPolicy>
class TMulticastDelegate<void(ParamTypes...), UserPolicy> : public TMulticastDelegateRegistration<void(ParamTypes...), UserPolicy>
{
private:
	using Super                         = TMulticastDelegateRegistration<void(ParamTypes...), UserPolicy>;
	using DelegateInstanceInterfaceType = typename Super::DelegateInstanceInterfaceType;

public:
	using RegistrationType = Super;

	TMulticastDelegate() = default;

	TMulticastDelegate(const TMulticastDelegate& Other)
	{
		*this = Other;
	}

	TMulticastDelegate& operator=(const TMulticastDelegate& Other)
	{
		if (&Other != this)
		{
			Super::template CopyFrom<DelegateInstanceInterfaceType>(Other);
		}
		return *this;
	}

	TMulticastDelegate(TMulticastDelegate&&) = default;
	TMulticastDelegate& operator=(TMulticastDelegate&&) = default;
	~TMulticastDelegate() = default;

	/**
	 * Broadcasts this delegate to all bound objects, except to those that may have expired.
	 *
	 * The constness of this method allows for broadcasting from const functions.
	 */
	void Broadcast(ParamTypes... Params) const
	{
		Super::Super::template Broadcast<typename Super::DelegateInstanceInterfaceType, ParamTypes...>(Params...);
	}
};

template <typename DelegateSignature>
using TTSMulticastDelegateRegistration = TMulticastDelegateRegistration<DelegateSignature, FDefaultTSDelegateUserPolicy>;

template <typename DelegateSignature>
using TTSMulticastDelegate = TMulticastDelegate<DelegateSignature, FDefaultTSDelegateUserPolicy>;


template <typename FuncType, typename ThreadSafetyMode = FNotThreadSafeDelegateMode>
class TDynamicMulticastDelegate;

template <typename MulticastDelegate, typename OwningClass, typename DelegateInfoClass>
struct TSparseDynamicDelegate;

/**
 * Dynamic delegate template class (UObject-based, serializable).  You'll use the various DECLARE_DYNAMIC_DELEGATE
 * macros to create the actual delegate type, templated to the function signature the delegate is compatible with.
 * Then, you can create an instance of that class when you want to assign functions to the delegate.
 */
template <typename FuncType, typename ThreadSafetyMode = FNotThreadSafeDelegateMode>
class TDynamicDelegate;

template <typename RetValType, typename... ParamTypes, typename ThreadSafetyMode>
class TDynamicDelegate<RetValType(ParamTypes...), ThreadSafetyMode> : public TScriptDelegate<ThreadSafetyMode>
{
	template <typename, typename>
	friend class TDynamicMulticastDelegate;

	/** Sparse version of TDynamicDelegate */
	template <typename, typename, typename>
	friend struct TSparseDynamicDelegate;

public:
	/**
	 * Default constructor
	 */
	TDynamicDelegate()
	{
	}

	/**
	 * Construction from an FScriptDelegate must be explicit.  This is really only used by UObject system internals.
	 *
	 * @param	InScriptDelegate	The delegate to construct from by copying
	 */
	explicit TDynamicDelegate(const TScriptDelegate<ThreadSafetyMode>& InScriptDelegate)
		: TScriptDelegate<ThreadSafetyMode>(InScriptDelegate)
	{
	}

	/**
	 * Templated helper class to define a typedef for user's method pointer, then used below
	 */
	template <bool bConst, typename UserClass, typename... VarTypes>
	struct TMethodPtrResolver
	{
		// The decay_t means that captured payloads will need to be passed by value, including things like FString.  Will need a mapping for that.
		using FMethodPtr = std::conditional_t<
			bConst,
			RetValType(UserClass::*)(ParamTypes... Params, std::decay_t<VarTypes>...) const,
			RetValType(UserClass::*)(ParamTypes... Params, std::decay_t<VarTypes>...)
		>;
	};

private:
	template <typename UserClass, typename... VarTypes>
	void BindDynamicImpl(const UserClass* InUserObject, FName InFunctionName, VarTypes&&... Vars)
	{
		static_assert(UE_USE_DYNAMIC_DELEGATE_PAYLOADS || sizeof...(VarTypes) == 0, "Payload support for dynamic delegates is disabled");

		check(InUserObject);

		// NOTE: If you hit a compile error on the following line, it means you're trying to use a non-UObject type
		//       with this delegate, which is not supported
		const UObject* UserObject = Cast<UObject>(InUserObject);

#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
		this->template BindProtected<RetValType(ParamTypes..., std::decay_t<VarTypes>...)>(UserObject, InFunctionName, Forward<VarTypes>(Vars)...);
#else
		this->BindProtected(UserObject, InFunctionName, Forward<VarTypes>(Vars)...);
#endif

		ensureMsgf(this->IsBound(), TEXT("Unable to bind delegate to '%s' (function might not be marked as a UFUNCTION or object may be pending kill)"), *InFunctionName.ToString());
	}

public:
	/**
	 * Binds a UObject instance and a UObject method address to this delegate.
	 *
	 * @param	InUserObject		UObject instance
	 * @param	InMethodPtr			Member function address pointer
	 * @param	InFunctionName		Name of member function, without class name
	 *
	 * NOTE:  Do not call this function directly.  Instead, call BindDynamic() which is a macro proxy function that
	 *        automatically sets the function name string for the caller.
	 *
	 * NOTE:  We're not actually using the incoming method pointer.  We simply require it for type-safety reasons.
	 */
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_BindDynamic(UserClass* InUserObject, typename TMethodPtrResolver<false, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		UE_STATIC_DEPRECATE(5.8, std::is_const_v<UserClass>, "Binding a delegate with a const object pointer and non-const function is deprecated.");
		this->BindDynamicImpl(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_BindDynamic(TObjectPtr<UserClass> InUserObject, typename TMethodPtrResolver<false, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		UE_STATIC_DEPRECATE(5.8, std::is_const_v<UserClass>, "Binding a delegate with a const object pointer and non-const function is deprecated.");
		this->BindDynamicImpl(ToRawPtr(InUserObject), InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_BindDynamic(UserClass* InUserObject, typename TMethodPtrResolver<true, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		this->BindDynamicImpl(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_BindDynamic(TObjectPtr<UserClass> InUserObject, typename TMethodPtrResolver<true, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		this->BindDynamicImpl(ToRawPtr(InUserObject), InFunctionName, Forward<VarTypes>(Vars)...);
	}

	/** Execute the delegate.  If the function pointer is not valid, an error will occur. */
	inline RetValType Execute(UE::Core::Private::TDelegateCallTraits_T<ParamTypes>... Params) const
	{
		/* Verify that the user object is still valid.  We only have a weak reference to it. */
		checkSlow(this->IsBound());
		return UE::Core::Private::TDelegateFunctionWrapper<RetValType, UE::Core::Private::TDelegateCallTraits_T<ParamTypes>...>::CallWrapper(*this, Params...);
	}

	/** Execute the delegate, but only if the function pointer is still valid */
	inline bool ExecuteIfBound(UE::Core::Private::TDelegateCallTraits_T<ParamTypes>... Params) const
		requires (UE::CVoid<RetValType>)
	{
		if (this->IsBound())
		{
			UE::Core::Private::TDelegateFunctionWrapper<RetValType, UE::Core::Private::TDelegateCallTraits_T<ParamTypes>...>::CallWrapper(*this, Params...);
			return true;
		}
		return false;
	}
};


/**
 * Dynamic multi-cast delegate template class (UObject-based, serializable).  You'll use the various
 * DECLARE_DYNAMIC_MULTICAST_DELEGATE macros to create the actual delegate type, templated to the function
 * signature the delegate is compatible with.   Then, you can create an instance of that class when you
 * want to assign functions to the delegate.
 */
template <typename RetValType, typename... ParamTypes, typename ThreadSafetyMode>
class TDynamicMulticastDelegate<RetValType(ParamTypes...), ThreadSafetyMode> : public TMulticastScriptDelegate<ThreadSafetyMode>
{
public:
	/** The actual single-cast delegate class for this multi-cast delegate */
	using FDelegate = TDynamicDelegate<RetValType(ParamTypes...), ThreadSafetyMode>;

	/**
	 * Default constructor
	 */
	TDynamicMulticastDelegate()
	{
	}

	/**
	 * Construction from an FMulticastScriptDelegate must be explicit.  This is really only used by UObject system internals.
	 *
	 * @param	InScriptDelegate	The delegate to construct from by copying
	 */
	explicit TDynamicMulticastDelegate(const TMulticastScriptDelegate<ThreadSafetyMode>& InMulticastScriptDelegate)
		: TMulticastScriptDelegate<ThreadSafetyMode>(InMulticastScriptDelegate)
	{
	}

private:
	template <typename UserClass, typename... VarTypes>
	bool IsAlreadyBoundImpl(const UserClass* InUserObject, FName InFunctionName, VarTypes&&... Vars) const
	{
		static_assert(UE_USE_DYNAMIC_DELEGATE_PAYLOADS || sizeof...(VarTypes) == 0, "Payload support for dynamic delegates is disabled");

		check(InUserObject);
		return this->Contains(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);
	}

public:
	/**
	 * Tests if a UObject instance and a UObject method address pair are already bound to this multi-cast delegate.
	 *
	 * @param	InUserObject		UObject instance
	 * @param	InMethodPtr			Member function address pointer
	 * @param	InFunctionName		Name of member function, without class name
	 * @return	True if the instance/method is already bound.
	 *
	 * NOTE:  Do not call this function directly.  Instead, call IsAlreadyBound() which is a macro proxy function that
	 *        automatically sets the function name string for the caller.
	 *
	 * NOTE:  We're not actually using the incoming method pointer.  We simply require it for type-safety reasons.
	 */
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE bool __Internal_IsAlreadyBound(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars) const
	{
		return this->IsAlreadyBoundImpl(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE bool __Internal_IsAlreadyBound(TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars) const
	{
		return this->IsAlreadyBoundImpl(ToRawPtr(InUserObject), InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE bool __Internal_IsAlreadyBound(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars) const
	{
		return this->IsAlreadyBoundImpl(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE bool __Internal_IsAlreadyBound(TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars) const
	{
		return this->IsAlreadyBoundImpl(ToRawPtr(InUserObject), InFunctionName, Forward<VarTypes>(Vars)...);
	}

private:
	template <typename UserClass, typename... VarTypes>
	void AddDynamicImpl(const UserClass* InUserObject, FName InFunctionName, VarTypes&&... Vars)
	{
		static_assert(UE_USE_DYNAMIC_DELEGATE_PAYLOADS || sizeof...(VarTypes) == 0, "Payload support for dynamic delegates is disabled");

		check(InUserObject);

		FDelegate NewDelegate;
		NewDelegate.BindDynamicImpl(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);

		this->Add(NewDelegate);
	}

public:
	/**
	 * Binds a UObject instance and a UObject method address to this multi-cast delegate.
	 *
	 * @param	InUserObject		UObject instance
	 * @param	InMethodPtr			Member function address pointer
	 * @param	InFunctionName		Name of member function, without class name
	 *
	 * NOTE:  Do not call this function directly.  Instead, call AddDynamic() which is a macro proxy function that
	 *        automatically sets the function name string for the caller.
	 *
	 * NOTE:  We're not actually using the incoming method pointer.  We simply require it for type-safety reasons.
	 */
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_AddDynamic(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		UE_STATIC_DEPRECATE(5.8, std::is_const_v<UserClass>, "Binding a delegate with a const object pointer and non-const function is deprecated.");
		this->AddDynamicImpl(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_AddDynamic(TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		UE_STATIC_DEPRECATE(5.8, std::is_const_v<UserClass>, "Binding a delegate with a const object pointer and non-const function is deprecated.");
		this->AddDynamicImpl(ToRawPtr(InUserObject), InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_AddDynamic(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		this->AddDynamicImpl(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_AddDynamic(TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		this->AddDynamicImpl(ToRawPtr(InUserObject), InFunctionName, Forward<VarTypes>(Vars)...);
	}

private:
	template <typename UserClass, typename... VarTypes>
	void AddUniqueDynamicImpl(const UserClass* InUserObject, FName InFunctionName, VarTypes&&... Vars)
	{
		static_assert(UE_USE_DYNAMIC_DELEGATE_PAYLOADS || sizeof...(VarTypes) == 0, "Payload support for dynamic delegates is disabled");

		check(InUserObject);

		FDelegate NewDelegate;
		NewDelegate.BindDynamicImpl(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);

		this->AddUnique(NewDelegate);
	}

public:
	/**
	 * Binds a UObject instance and a UObject method address to this multi-cast delegate, but only if it hasn't been bound before.
	 *
	 * @param	InUserObject		UObject instance
	 * @param	InMethodPtr			Member function address pointer
	 * @param	InFunctionName		Name of member function, without class name
	 *
	 * NOTE:  Do not call this function directly.  Instead, call AddUniqueDynamic() which is a macro proxy function that
	 *        automatically sets the function name string for the caller.
	 *
	 * NOTE:  We're not actually using the incoming method pointer.  We simply require it for type-safety reasons.
	 */
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_AddUniqueDynamic(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		UE_STATIC_DEPRECATE(5.8, std::is_const_v<UserClass>, "Binding a delegate with a const object pointer and non-const function is deprecated.");
		this->AddUniqueDynamicImpl(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_AddUniqueDynamic(TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		UE_STATIC_DEPRECATE(5.8, std::is_const_v<UserClass>, "Binding a delegate with a const object pointer and non-const function is deprecated.");
		this->AddUniqueDynamicImpl(ToRawPtr(InUserObject), InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_AddUniqueDynamic(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		this->AddUniqueDynamicImpl(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_AddUniqueDynamic(TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		this->AddUniqueDynamicImpl(ToRawPtr(InUserObject), InFunctionName, Forward<VarTypes>(Vars)...);
	}

private:
	template <typename UserClass, typename... VarTypes>
	void RemoveDynamicImpl(const UserClass* InUserObject, FName InFunctionName, VarTypes&&... Vars)
	{
		static_assert(UE_USE_DYNAMIC_DELEGATE_PAYLOADS || sizeof...(VarTypes) == 0, "Payload support for dynamic delegates is disabled");

		check(InUserObject);

		this->Remove(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);
	}

public:
	/**
	 * Unbinds a UObject instance and a UObject method address from this multi-cast delegate.
	 *
	 * @param	InUserObject		UObject instance
	 * @param	InMethodPtr			Member function address pointer
	 * @param	InFunctionName		Name of member function, without class name
	 *
	 * NOTE:  Do not call this function directly.  Instead, call RemoveDynamic() which is a macro proxy function that
	 *        automatically sets the function name string for the caller.
	 *
	 * NOTE:  We're not actually using the incoming method pointer.  We simply require it for type-safety reasons.
	 */
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_RemoveDynamic(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		this->RemoveDynamicImpl(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_RemoveDynamic( TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass, VarTypes&&...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		this->RemoveDynamicImpl(ToRawPtr(InUserObject), InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_RemoveDynamic(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass, VarTypes...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		this->RemoveDynamicImpl(InUserObject, InFunctionName, Forward<VarTypes>(Vars)...);
	}
	template <typename UserClass, typename... VarTypes>
	UE_REWRITE void __Internal_RemoveDynamic(TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass, VarTypes&&...>::FMethodPtr InMethodPtr, FName InFunctionName, VarTypes&&... Vars)
	{
		this->RemoveDynamicImpl(ToRawPtr(InUserObject), InFunctionName, Forward<VarTypes>(Vars)...);
	}

	/** Broadcasts this delegate to all bound objects, except to those that may have expired */
	inline void Broadcast(UE::Core::Private::TDelegateCallTraits_T<ParamTypes>... Params) const
	{
		UE::Core::Private::TDelegateFunctionWrapper<RetValType, UE::Core::Private::TDelegateCallTraits_T<ParamTypes>...>::CallWrapper(*this, Params...);
	}
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Templates/IsConst.h"
#include "Templates/RemoveReference.h"
#endif
