// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Concepts/Pointer.h"
#include "Concepts/PointerLike.h"
#include "Templates/TypeHash.h" 
#include "UObject/NameTypes.h"
#include "Delegates/DelegateSettings.h"

/**
 * Class representing a handle to a specific object/function pair bound to a delegate.
 */
class FDelegateHandle
{
public:
	enum EGenerateNewHandleType
	{
		GenerateNewHandle
	};

	/** Creates an initially unset handle */
	FDelegateHandle()
		: ID(0)
	{
	}

	/** Creates a handle pointing to a new instance */
	explicit FDelegateHandle(EGenerateNewHandleType)
		: ID(GenerateNewID())
	{
	}

	/** Returns true if this was ever bound to a delegate, but you need to check with the owning delegate to confirm it is still valid */
	bool IsValid() const
	{
		return ID != 0;
	}

	/** Clear handle to indicate it is no longer bound */
	void Reset()
	{
		ID = 0;
	}

	bool operator==(const FDelegateHandle& Rhs) const
	{
		return ID == Rhs.ID;
	}

	bool operator!=(const FDelegateHandle& Rhs) const
	{
		return ID != Rhs.ID;
	}

	UE_REWRITE friend uint32 GetTypeHash(const FDelegateHandle& Key)
	{
		return GetTypeHash(Key.ID);
	}

private:

	/**
	 * Generates a new ID for use the delegate handle.
	 *
	 * @return A unique ID for the delegate.
	 */
	static CORE_API uint64 GenerateNewID();

	uint64 ID;
};

#if UE_WITH_REMOTE_OBJECT_HANDLE
/*
 * Helper class that validates Delegate object type
 */
template <bool bConst>
class TDelegateUserObject
{
	template <bool>
	friend class TDelegateUserObject;

	using VoidType = std::conditional_t<bConst, const void, void>;
	using UObjectType = std::conditional_t<bConst, const UObject, UObject>;

	VoidType* UserObject;
	bool bIsUObject = false;

private:
	explicit TDelegateUserObject(VoidType* InUserObject, bool bInIsUObject)
		: UserObject(InUserObject)
		, bIsUObject(bInIsUObject)
	{
	}

public:
	template<typename UserObjectType>
		requires (bConst || !std::is_const_v<UserObjectType>)
	TDelegateUserObject(UserObjectType* InUserObject)
	{
		UE_STATIC_ASSERT_COMPLETE_TYPE(UserObjectType, "TDelegateUserObject(UserObject) UserObject must not be an incomplete type");
		UserObject = (VoidType*)InUserObject;
		bIsUObject = std::is_convertible_v<UserObjectType*, UObjectType*>;
	}
	template<UE::CPointerLike PtrWrapperType>
		requires (!UE::CPointer<PtrWrapperType>)
	UE_REWRITE TDelegateUserObject(const PtrWrapperType& InPtrWrapper)
		: TDelegateUserObject(InPtrWrapper ? &*InPtrWrapper : nullptr)
	{
	}
	UE_REWRITE bool IsUObject() const
	{
		return bIsUObject;
	}
	UE_REWRITE VoidType* GetRaw() const
	{
		return UserObject;
	}
	UE_REWRITE UObjectType* GetUObject() const
	{
		return (UObjectType*)UserObject;
	}
	UE_REWRITE operator VoidType*() const
	{
		return UserObject;
	}
	UE_REWRITE operator TDelegateUserObject<true>() const
		requires (!bConst)
	{
		return TDelegateUserObject<true>(UserObject, bIsUObject);
	}
};
using FDelegateUserObject = TDelegateUserObject<false>;
using FDelegateUserObjectConst = TDelegateUserObject<true>;
#else
using FDelegateUserObject = void*;
using FDelegateUserObjectConst = const void*;
#endif

class IDelegateInstance
{
public:
	/**
	 * Virtual destructor.
	 */
	virtual ~IDelegateInstance() = default;

#if USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME

	/**
	 * Tries to return the name of a bound function.  Returns NAME_None if the delegate is unbound or
	 * a binding name is unavailable.
	 *
	 * Note: Only intended to be used to aid debugging of delegates.
	 *
	 * @return The name of the bound function, NAME_None if no name was available.
	 */
	virtual FName TryGetBoundFunctionName() const = 0;

#endif

	/**
	 * Returns the UObject that this delegate instance is bound to.
	 *
	 * @return Pointer to the UObject, or nullptr if not bound to a UObject.
	 */
	virtual UObject* GetUObject( ) const = 0;

	/**
	 * Returns a pointer to an object bound to this delegate instance, intended for quick lookup in the timer manager,
	 *
	 * @return A pointer to an object referenced by the delegate instance.
	 */
	virtual const void* GetObjectForTimerManager() const = 0;

	/**
	 * Returns the address of the method pointer which can be used to learn the address of the function that will be executed.
	 * Returns nullptr if this delegate type does not directly invoke a function pointer.
	 *
	 * Note: Only intended to be used to aid debugging of delegates.
	 *
	 * @return The address of the function pointer that would be executed by this delegate
	 */
	virtual uint64 GetBoundProgramCounterForTimerManager() const = 0;

	/**
	 * Returns true if this delegate is bound to the specified UserObject,
	 *
	 * Deprecated.
	 *
	 * @param InUserObject
	 *
	 * @return True if delegate is bound to the specified UserObject
	 */
	virtual bool HasSameObject(FDelegateUserObjectConst InUserObject) const = 0;

	/**
	 * Checks to see if the user object bound to this delegate can ever be valid again.
	 * used to compact multicast delegate arrays so they don't expand without limit.
	 *
	 * @return True if the user object can never be used again
	 */
	virtual bool IsCompactable( ) const
	{
		return !IsSafeToExecute();
	}

	/**
	 * Checks to see if the user object bound to this delegate is still valid
	 *
	 * @return True if the user object is still valid and it's safe to execute the function call
	 */
	virtual bool IsSafeToExecute( ) const = 0;

	/**
	 * Returns a handle for the delegate.
	 */
	virtual FDelegateHandle GetHandle() const = 0;
};
