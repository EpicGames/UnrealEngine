// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

template <typename T, T... Indices>
struct TIntegerSequence
{
};

#ifdef _MSC_VER

template <typename T, T N>
using TMakeIntegerSequence = __make_integer_seq<TIntegerSequence, T, N>;

#elif __has_builtin(__make_integer_seq)

template <typename T, T N>
using TMakeIntegerSequence = __make_integer_seq<TIntegerSequence, T, N>;

#else

namespace UE4IntegerSequence_Private
{
	template <typename T, unsigned N>
	struct TMakeIntegerSequenceImpl;
}

template <typename T, T N>
using TMakeIntegerSequence = typename UE4IntegerSequence_Private::TMakeIntegerSequenceImpl<T, N>::Type;

	// Doxygen can't parse recursive template definitions; just skip it.
	#if !UE_BUILD_DOCS

	namespace UE4IntegerSequence_Private
	{
		template<unsigned N, typename T1, typename T2>
		struct TConcatImpl;

		template<unsigned N, typename T, T... Indices1, T... Indices2>
		struct TConcatImpl<N, TIntegerSequence<T, Indices1...>, TIntegerSequence<T, Indices2...>> : TIntegerSequence<T, Indices1..., (T(N + Indices2))...>
		{
			using Type = TIntegerSequence<T, Indices1..., (T(N + Indices2))...>;
		};

		template<unsigned N, typename T1, typename T2>
		using TConcat = typename TConcatImpl<N, T1, T2>::Type;

		template <typename T, unsigned N>
		struct TMakeIntegerSequenceImpl : TConcat<N / 2, TMakeIntegerSequence<T, N / 2>, TMakeIntegerSequence<T, N - N / 2>>
		{
			using Type = TConcat<N / 2, TMakeIntegerSequence<T, N / 2>, TMakeIntegerSequence<T, N - N / 2>>;
		};

		template <typename T>
		struct TMakeIntegerSequenceImpl<T, 1> : TIntegerSequence<T, T(0)> 
		{
			using Type = TIntegerSequence<T, T(0)>;
		};

		template <typename T>
		struct TMakeIntegerSequenceImpl<T, 0> : TIntegerSequence<T> 
		{
			using Type = TIntegerSequence<T>;
		};
	};

	#endif
#endif