// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM.h"
#include "CoreTypes.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Array.h"
#include "Math/UnrealMathUtility.h"
#include "Delegates/IDelegateInstance.h" // IWYU pragma: export
#include "Delegates/DelegateBase.h" // IWYU pragma: export

#if !defined(NUM_MULTICAST_DELEGATE_INLINE_ENTRIES) || NUM_MULTICAST_DELEGATE_INLINE_ENTRIES == 0
	typedef FHeapAllocator FMulticastInvocationListAllocatorType;
#elif NUM_MULTICAST_DELEGATE_INLINE_ENTRIES < 0
	#error NUM_MULTICAST_DELEGATE_INLINE_ENTRIES must be positive
#else
	typedef TInlineAllocator<NUM_MULTICAST_DELEGATE_INLINE_ENTRIES> FMulticastInvocationListAllocatorType;
#endif

#define UE_MULTICAST_DELEGATE_DEFAULT_COMPACTION_THRESHOLD 2

#if UE_DELEGATE_CHECK_LIFETIME

// Centralized tracker for checking the lifetime of module-bound delegates registered to a multicast
class FMulticastDelegateTracker
{
public:

	// Register a delegate instance for a given module
	static CORE_API void RegisterDelegateInstance(FDelegateHandle DelegateHandle, FName ModuleName, FName CallsiteName);

	// Unregister a delegate instance 
	static CORE_API void UnregisterDelegateInstance(FDelegateHandle DelegateHandle);

	// Report stale delegates for a list of modules
	static CORE_API void CheckForStaleDelegatesInModules(TConstArrayView<FName> ModuleNames);
};

#  define UE_UNREGISTER_MULTICAST_INSTANCE(Handle) FMulticastDelegateTracker::UnregisterDelegateInstance(Handle)

#else
#  define UE_UNREGISTER_MULTICAST_INSTANCE(Handle)
#endif // UE_DELEGATE_CHECK_LIFETIME

/**
 * Abstract base class for multicast delegates.
 */
template<typename UserPolicy>
class TMulticastDelegateBase :
	public TDelegateAccessHandlerBase<typename UserPolicy::FThreadSafetyMode>
{
protected:
	using Super = TDelegateAccessHandlerBase<typename UserPolicy::FThreadSafetyMode>;
	using typename Super::FReadAccessScope;
	using Super::GetReadAccessScope;
	using typename Super::FWriteAccessScope;
	using Super::GetWriteAccessScope;

	// individual bindings are not checked for races as it's done for the parent delegate
	using UnicastDelegateType = TDelegateBase<FNotThreadSafeNotCheckedDelegateMode>;

	using InvocationListType = TArray<UnicastDelegateType, FMulticastInvocationListAllocatorType>;

public:
	TMulticastDelegateBase(TMulticastDelegateBase&& Other)
	{
		*this = MoveTemp(Other);
	}

#if UE_DELEGATE_CHECK_LIFETIME
	~TMulticastDelegateBase()
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		// We need multicasts to clean up after themselves so that unregistrations can be tracked
		for (const UnicastDelegateType& DelegateBaseRef : InvocationList)
		{
			if (const IDelegateInstance* DelegateInstance = DelegateBaseRef.GetDelegateInstanceProtected())
			{
				UE_UNREGISTER_MULTICAST_INSTANCE(DelegateInstance->GetHandle());
			}
		}
	}
#endif // UE_DELEGATE_CHECK_LIFETIME

	TMulticastDelegateBase& operator=(TMulticastDelegateBase&& Other)
	{
		if (&Other == this)
		{
			return *this;
		}

		InvocationListType LocalInvocationList;
		int32 LocalCompactionThreshold;

		{
			FWriteAccessScope OtherWriteScope = Other.GetWriteAccessScope();

			LocalInvocationList = MoveTemp(Other.InvocationList);
			LocalCompactionThreshold = Other.CompactionThreshold;
			Other.CompactionThreshold = UE_MULTICAST_DELEGATE_DEFAULT_COMPACTION_THRESHOLD;
			checkf(Other.InvocationListLockCount == 0, TEXT("Moving from a multicast delegate while it is mid broadcast"));
		}

		{
			FWriteAccessScope ThisWriteScope = GetWriteAccessScope();

			ClearUnchecked();
			InvocationList = MoveTemp(LocalInvocationList);
			CompactionThreshold = LocalCompactionThreshold;
			checkf(InvocationListLockCount == 0, TEXT("Moving to a multicast delegate while it is mid broadcast"));
		}

		return *this;
	}

	/** Removes all functions from this delegate's invocation list. */
	void Clear( )
	{
		FWriteAccessScope WriteScope = GetWriteAccessScope();

		ClearUnchecked();
	}

	/**
	 * Checks to see if any functions are bound to this multi-cast delegate.
	 *
	 * @return true if any functions are bound, false otherwise.
	 */
	inline bool IsBound( ) const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();
	
		for (const UnicastDelegateType& DelegateBaseRef : InvocationList)
		{
			if (DelegateBaseRef.GetDelegateInstanceProtected())
			{
				return true;
			}
		}
		return false;
	}

	/** 
	 * Checks to see if any functions are bound to the given user object.
	 *
	 * @return	True if any functions are bound to InUserObject, false otherwise.
	 */
	inline bool IsBoundToObject(FDelegateUserObjectConst InUserObject) const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		for (const UnicastDelegateType& DelegateBaseRef : InvocationList)
		{
			const IDelegateInstance* DelegateInstance = DelegateBaseRef.GetDelegateInstanceProtected();
			if ((DelegateInstance != nullptr) && DelegateInstance->HasSameObject(InUserObject))
			{
				return true;
			}
		}

		return false;
	}

	/**
	 * Removes all functions from this multi-cast delegate's invocation list that are bound to the specified UserObject.
	 * Note that the order of the delegates may not be preserved!
	 *
	 * @param InUserObject The object to remove all delegates for.
	 * @return  The number of delegates successfully removed.
	 */
	int32 RemoveAll(FDelegateUserObjectConst InUserObject)
	{
		FWriteAccessScope WriteScope = GetWriteAccessScope();

		int32 Result = 0;
		if (ShouldSkipCompactionUnderAutoRTFM() || (InvocationListLockCount > 0))
		{
			for (UnicastDelegateType& DelegateBaseRef : InvocationList)
			{
				IDelegateInstance* DelegateInstance = DelegateBaseRef.GetDelegateInstanceProtected();
				if ((DelegateInstance != nullptr) && DelegateInstance->HasSameObject(InUserObject))
				{
					UE_UNREGISTER_MULTICAST_INSTANCE(DelegateInstance->GetHandle());

					// Manually unbind the delegate here so the compaction will find and remove it.
					DelegateBaseRef.Unbind();
					++Result;
				}
			}

			// can't compact at the moment as the invocation list is locked, but set out threshold to zero so the next add will do it
			if (Result > 0)
			{
				CompactionThreshold = 0;
			}
		}
		else
		{
			// compact us while shuffling in later delegates to fill holes
			for (int32 InvocationListIndex = 0; InvocationListIndex < InvocationList.Num();)
			{
				UnicastDelegateType& DelegateBaseRef = InvocationList[InvocationListIndex];

				IDelegateInstance* DelegateInstance = DelegateBaseRef.GetDelegateInstanceProtected();

				if (DelegateInstance == nullptr
					|| DelegateInstance->HasSameObject(InUserObject)
					|| DelegateInstance->IsCompactable())
				{
					if (DelegateInstance)
					{
						UE_UNREGISTER_MULTICAST_INSTANCE(DelegateInstance->GetHandle());
					}

					InvocationList.RemoveAtSwap(InvocationListIndex, EAllowShrinking::No);
					++Result;
				}
				else
				{
					InvocationListIndex++;
				}
			}

			CompactionThreshold = FMath::Max(UE_MULTICAST_DELEGATE_DEFAULT_COMPACTION_THRESHOLD, 2 * InvocationList.Num());

			InvocationList.Shrink();
		}

		return Result;
	}

	/**
	 * Returns the amount of memory allocated by this delegate's invocation list and the delegates stored within it, not including sizeof(*this).
	 */
	SIZE_T GetAllocatedSize() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();
	
		SIZE_T Size = 0;
		Size += InvocationList.GetAllocatedSize();
		for (const UnicastDelegateType& DelegateBaseRef : InvocationList)
		{
			Size += DelegateBaseRef.GetAllocatedSize();
		}
		return Size;
	}

protected:

	/** Hidden default constructor. */
	inline TMulticastDelegateBase( )
		: CompactionThreshold(UE_MULTICAST_DELEGATE_DEFAULT_COMPACTION_THRESHOLD)
		, InvocationListLockCount(0)
	{
	}

protected:
	template<typename DelegateInstanceInterfaceType>
	void CopyFrom(const TMulticastDelegateBase& Other)
	{
		InvocationListType TempInvocationList;

		{
			FReadAccessScope OtherReadScope = Other.GetReadAccessScope();

		    for (const UnicastDelegateType& OtherDelegateRef : Other.GetInvocationList())
		    {
				if (const IDelegateInstance* OtherInstance = OtherDelegateRef.GetDelegateInstanceProtected())
			    {
				    UnicastDelegateType TempDelegate;
					static_cast<const DelegateInstanceInterfaceType*>(OtherInstance)->CreateCopy(FPrivateDelegateAllocation{TempDelegate});
				    TempInvocationList.Add(MoveTemp(TempDelegate));
			    }
		    }
		}

		{
			FWriteAccessScope ThisWriteScope = GetWriteAccessScope();

			ClearUnchecked();
			InvocationList = MoveTemp(TempInvocationList);
		}
	}

	template<typename DelegateInstanceInterfaceType, typename... ParamTypes>
	void Broadcast(ParamTypes... Params) const
	{
		// the `const` on the method is a lie
		FWriteAccessScope WriteScope = const_cast<TMulticastDelegateBase*>(this)->GetWriteAccessScope();

		bool NeedsCompaction = false;

		LockInvocationList();
		{
			const InvocationListType& LocalInvocationList = GetInvocationList();

			// call bound functions in reverse order, so we ignore any instances that may be added by callees
			for (int32 InvocationListIndex = LocalInvocationList.Num() - 1; InvocationListIndex >= 0; --InvocationListIndex)
			{
				// this down-cast is OK! allows for managing invocation list in the base class without requiring virtual functions
				const UnicastDelegateType& DelegateBase = LocalInvocationList[InvocationListIndex];

				const IDelegateInstance* DelegateInstanceInterface = DelegateBase.GetDelegateInstanceProtected();
				if (DelegateInstanceInterface == nullptr || !static_cast<const DelegateInstanceInterfaceType*>(DelegateInstanceInterface)->ExecuteIfSafe(Params...))
				{
					NeedsCompaction = true;
				}
			}
		}
		UnlockInvocationList();

		if (NeedsCompaction)
		{
			const_cast<TMulticastDelegateBase*>(this)->CompactInvocationList();
		}
	}

	/**
	 * Adds the given delegate instance to the invocation list.
	 *
	 * @param NewDelegateBaseRef The delegate instance to add.
	 */
	template <typename NewDelegateType>
	inline FDelegateHandle AddDelegateInstance(NewDelegateType&& NewDelegateBaseRef)
	{
		FWriteAccessScope WriteScope = GetWriteAccessScope();

		FDelegateHandle Result;
		if (NewDelegateBaseRef.IsBound())
		{
			// compact but obey threshold of when this will trigger
			CompactInvocationList(true);
			Result = NewDelegateBaseRef.GetHandle();
			InvocationList.Emplace(Forward<NewDelegateType>(NewDelegateBaseRef));
		}
		return Result;
	}

	/**
	 * Removes a function from this multi-cast delegate's invocation list (performance is O(N)).
	 *
	 * @param Handle The handle of the delegate instance to remove.
	 * @return  true if the delegate was successfully removed.
	 */
	bool RemoveDelegateInstance(FDelegateHandle Handle)
	{
		FWriteAccessScope WriteScope = GetWriteAccessScope();

		UE_UNREGISTER_MULTICAST_INSTANCE(Handle);

		for (int32 InvocationListIndex = 0; InvocationListIndex < InvocationList.Num(); ++InvocationListIndex)
		{
			UnicastDelegateType& DelegateBase = InvocationList[InvocationListIndex];

			IDelegateInstance* DelegateInstance = DelegateBase.GetDelegateInstanceProtected();
			if (DelegateInstance && DelegateInstance->GetHandle() == Handle)
			{
				DelegateBase.Unbind();
				CompactInvocationList();
				return true; // each delegate binding has a unique handle, so once we find it, we can stop
			}
		}

		return false;
	}

private:

	inline bool ShouldSkipCompactionUnderAutoRTFM() const
	{
		// As a performance optimization for the AutoRTFM runtime, we *always*
		// skip compacting a multicast delegate when called via the AutoRTFM
		// runtime. We skip calls both in the bodies of transactions themselves,
		// and also those that are called as part of on-commit or on-abort calls.
		return !AutoRTFM::IsIdle();
	}

	/**
	 * Removes any expired or deleted functions from the invocation list.
	 *
	 * @see RequestCompaction
	 */
	void CompactInvocationList(bool CheckThreshold=false)
	{
		if (ShouldSkipCompactionUnderAutoRTFM())
		{
			return;
		}

		// if locked and no object, just return
		if (InvocationListLockCount > 0)
		{
			return;
		}

		// if checking threshold, obey but decay. This is to ensure that even infrequently called delegates will
		// eventually compact during an Add()
		if (CheckThreshold 	&& --CompactionThreshold > InvocationList.Num())
		{
			return;
		}

		int32 OldNumItems = InvocationList.Num();

		// Find anything null or compactable and remove it
		for (int32 InvocationListIndex = 0; InvocationListIndex < InvocationList.Num();)
		{
			auto& DelegateBaseRef = InvocationList[InvocationListIndex];

			IDelegateInstance* DelegateInstance = DelegateBaseRef.GetDelegateInstanceProtected();

			if (DelegateInstance == nullptr	|| DelegateInstance->IsCompactable())
			{
				if (DelegateInstance)
				{
					UE_UNREGISTER_MULTICAST_INSTANCE(DelegateInstance->GetHandle());
				}

				InvocationList.RemoveAtSwap(InvocationListIndex, EAllowShrinking::No);
			}
			else
			{
				InvocationListIndex++;
			}
		}

		CompactionThreshold = FMath::Max(UE_MULTICAST_DELEGATE_DEFAULT_COMPACTION_THRESHOLD, 2 * InvocationList.Num());

		if (OldNumItems > CompactionThreshold)
		{
			// would be nice to shrink down to threshold, but reserve only grows..?
			InvocationList.Shrink();
		}
	}

	/**
	 * Gets a read-only reference to the invocation list.
	 *
	 * @return The invocation list.
	 */
	inline InvocationListType& GetInvocationList( )
	{
		return InvocationList;
	}

	inline const InvocationListType& GetInvocationList( ) const
	{
		return InvocationList;
	}

	/** Increments the lock counter for the invocation list. */
	inline void LockInvocationList( ) const
	{
		if (ShouldSkipCompactionUnderAutoRTFM())
		{
			return;
		}
		
		++InvocationListLockCount;
	}

	/** Decrements the lock counter for the invocation list. */
	inline void UnlockInvocationList( ) const
	{
		if (ShouldSkipCompactionUnderAutoRTFM())
		{
			return;
		}
		
		--InvocationListLockCount;
	}

	/** Returns the lock counter for the invocation list. */
	inline int32 GetInvocationListLockCount() const
	{
		return InvocationListLockCount;
	}

private:
	void ClearUnchecked()
	{
		for (UnicastDelegateType& DelegateBaseRef : InvocationList)
		{
			UE_UNREGISTER_MULTICAST_INSTANCE(DelegateBaseRef.GetHandle());

			DelegateBaseRef.Unbind();
		}

		CompactInvocationList(false);
	}

	/** Holds the collection of delegate instances to invoke. */
	InvocationListType InvocationList;

	/** Used to determine when a compaction should happen. */
	int32 CompactionThreshold = UE_MULTICAST_DELEGATE_DEFAULT_COMPACTION_THRESHOLD;

	/** Holds a lock counter for the invocation list. */
	mutable int32 InvocationListLockCount = 0;
};
