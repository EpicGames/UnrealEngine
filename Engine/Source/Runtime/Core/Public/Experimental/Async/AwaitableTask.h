// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Async/Fundamental/Task.h"
#include "Async/Fundamental/Scheduler.h"
#include "Async/ManualResetEvent.h"
#include "Misc/Launder.h"

template<typename>
class TAwaitableTask;

namespace AwaitableTask_Detail
{
	template<typename ReturnType>
	inline ReturnType MakeDummyValue()
	{
		return *(reinterpret_cast<ReturnType*>(uintptr_t(1)));
	}

	template<>
	inline void MakeDummyValue<void>()
	{
		return;
	}

	class FPromiseBase 
	{
	};

	//the TPromiseVTableBase type is used to move the vtable pointer out of the promise, where it otherwise could interfere the alignment of the Callable/Lambda.
	template<typename ReturnType>
	class TPromiseVTableBase
	{
	protected:
		FPromiseBase* Promise = nullptr;
		TPromiseVTableBase(FPromiseBase* InPromise) : Promise(InPromise)
		{}

	public:
		TPromiseVTableBase() = default;

		virtual bool TryLaunch() 
		{ 
			checkNoEntry(); //if we get to this place than the compiler is optimizing our vtable lookup and ignores our undefined/shallow copy of the memory
			return false; 
		}

		virtual void IncrementRefCount() 
		{ 
			checkNoEntry(); //if we get to this place than the compiler is optimizing our vtable lookup and ignores our undefined/shallow copy of the memory
		}

		virtual ReturnType GetResult() 
		{ 
			checkNoEntry(); //if we get to this place than the compiler is optimizing our vtable lookup and ignores our undefined/shallow copy of the memory
			return MakeDummyValue<ReturnType>(); 
		}

		virtual bool IsLaunched() const 
		{ 
			checkNoEntry(); //if we get to this place than the compiler is optimizing our vtable lookup and ignores our undefined/shallow copy of the memory
			return false; 
		}

		virtual bool IsValid() const 
		{ 
			checkNoEntry(); //if we get to this place than the compiler is optimizing our vtable lookup and ignores our undefined/shallow copy of the memory
			return false; 
		}

		virtual bool IsCompleted() const 
		{ 
			checkNoEntry(); //if we get to this place than the compiler is optimizing our vtable lookup and ignores our undefined/shallow copy of the memory 
			return true; 
		}

		virtual void Finish() 
		{ 
			checkNoEntry(); //if we get to this place than the compiler is optimizing our vtable lookup and ignores our undefined/shallow copy of the memory
		}

		TPromiseVTableBase& operator= (const TPromiseVTableBase& Other);
		TPromiseVTableBase& operator= (TPromiseVTableBase&& Other);
	};

	//the TPromiseVTableDummy is used with empty handles, this allows us to always call the virtual interface without nullpointer checks
	//validation for launching and getting the result of empty taskhandles is done as well.
	template<typename ReturnType>
	class TPromiseVTableDummy final : public TPromiseVTableBase<ReturnType>
	{
	public:
		TPromiseVTableDummy() : TPromiseVTableBase<ReturnType>(nullptr)
		{
			static_assert(sizeof(TPromiseVTableDummy) == sizeof(TPromiseVTableBase<ReturnType>), "Sizes must match because we reinterpret the memory");
		}

		bool TryLaunch() override 
		{ 
			checkf(false, TEXT("trying to Launch an empty AwaitableTask")); 
			return false; 
		}

		void IncrementRefCount() override 
		{ 
		}

		ReturnType GetResult() override 
		{ 
			checkf(false, TEXT("trying to get the Result of an empty AwaitableTask")); 
			return MakeDummyValue<ReturnType>(); 
		}

		bool IsLaunched() const override 
		{ 
			return false; 
		}

		bool IsValid() const override 
		{ 
			return false; 
		}

		bool IsCompleted() const override 
		{ 
			return true; 
		}

		void Finish() override 
		{ 
		}
	};

	template<typename ReturnType>
	inline TPromiseVTableBase<ReturnType>& TPromiseVTableBase<ReturnType>::operator= (const TPromiseVTableBase<ReturnType>& Other)
	{
		memcpy((void*)this, (void*)&Other, sizeof(TPromiseVTableBase<ReturnType>)); //-V598
		return *this;
	}

	template<typename ReturnType>
	inline TPromiseVTableBase<ReturnType>& TPromiseVTableBase<ReturnType>::operator= (TPromiseVTableBase<ReturnType>&& Other)
	{
		memcpy((void*)this, (void*)&Other, sizeof(TPromiseVTableBase<ReturnType>)); //-V598
		new (&Other) TPromiseVTableDummy<ReturnType>();
		return *this;
	}

	//the TPromiseVTable is used to type erase the PromiseType and therefore its lambda type
	template<typename PromiseType>
	class TPromiseVTable final : public TPromiseVTableBase<typename PromiseType::ReturnType>
	{
		using ReturnType = typename PromiseType::ReturnType;

	public:
		TPromiseVTable(FPromiseBase* InPromise) : TPromiseVTableBase<ReturnType>(InPromise)
		{
			checkSlow(this->Promise);
			static_assert(sizeof(TPromiseVTable) == sizeof(TPromiseVTableBase<ReturnType>), "Sizes must match because we reinterpret the memory");
		}

		bool TryLaunch() override
		{
			PromiseType* LocalPromise = static_cast<PromiseType*>(this->Promise);
			return LocalPromise->TryLaunch();
		}

		void IncrementRefCount() override 
		{
			PromiseType* LocalPromise = static_cast<PromiseType*>(this->Promise);
			LocalPromise->IncrementRefCount();
		}

		ReturnType GetResult() override
		{
			PromiseType* LocalPromise = static_cast<PromiseType*>(this->Promise);
			return LocalPromise->GetResult();
		}

		bool IsLaunched() const override
		{ 
			PromiseType* LocalPromise = static_cast<PromiseType*>(this->Promise);
			return LocalPromise->IsLaunched();
		}

		bool IsValid() const override
		{ 
			return true; 
		}

		bool IsCompleted() const override
		{
			PromiseType* LocalPromise = static_cast<PromiseType*>(this->Promise);
			return LocalPromise->IsCompleted();
		}

		void Finish() override
		{ 
			PromiseType* LocalPromise = static_cast<PromiseType*>(this->Promise);
			new (this) TPromiseVTableDummy<ReturnType>();
			LocalPromise->Finish();
		}
	};

	template<typename TReturnType, typename CallableType>
	class TPromise final : public FPromiseBase
	{
	public:
		using ReturnType = TReturnType;

	private:
		using ThisType = TPromise<ReturnType, CallableType>;

	private:
		CallableType Callable;
		LowLevelTasks::FTask Task;
		ReturnType ReturnValue;
		std::atomic_int ReferenceCounter{ 2 }; //starts at 2 because at construction the AwaitableTask will hold a Reference and the LowLevelTasks::FTask does as well.
		UE::FManualResetEvent Completed;

		inline void Execute()
		{
			ReturnValue = Invoke(Callable);
			Completed.Notify();
		}

		TPromise(const TPromise&) = delete;
		TPromise& operator= (const TPromise& Other) = delete;

	public:
		template<typename CallableT>
		TPromise(const TCHAR* DebugName, LowLevelTasks::ETaskPriority Priority, CallableT&& InCallable) : Callable(Forward<CallableT>(InCallable))
		{
			Task.Init(DebugName, Priority, [this, Deleter(LowLevelTasks::TDeleter<ThisType, &ThisType::Finish>(this))]()
			{
				Execute();
			});
		}

		inline bool TryLaunch()
		{
			return LowLevelTasks::TryLaunch(Task);
		}

		inline bool IsLaunched() const
		{ 
			return !Task.IsReady();
		}

		inline void IncrementRefCount()
		{
			ReferenceCounter.fetch_add(1, std::memory_order_release);
		}

		inline bool IsCompleted() const
		{
			return Completed.IsNotified();
		}

		inline void Finish()
		{
			int LocalCounter = ReferenceCounter.fetch_sub(1, std::memory_order_release);
			if (LocalCounter == 1) //this is 1 because fetch_sub returns the value before the decrement (value is actually 0)
			{
				delete this;
			}
		}

		inline ReturnType GetResult()
		{
			if (Task.TryCancel())
			{
				Execute();
				return ReturnValue;
			}
			else
			{
				Completed.Wait();
				return ReturnValue;
			}
		}
	};

	template<typename CallableType>
	class TPromise<void, CallableType> final : public FPromiseBase
	{
	public:
		using ReturnType = void;

	private:
		using ThisType = TPromise<void, CallableType>;

	private:
		CallableType Callable;
		LowLevelTasks::FTask Task;
		std::atomic_int ReferenceCounter{ 2 }; //starts at 2 because at construction the AwaitableTask will hold a Reference and the LowLevelTasks::FTask does as well.
		UE::FManualResetEvent Completed;

		inline void Execute()
		{
			Invoke(Callable);
			Completed.Notify();
		}

		TPromise(const TPromise&) = delete;
		TPromise& operator= (const TPromise& Other) = delete;

	public:
		template<typename CallableT>
		TPromise(const TCHAR* DebugName, LowLevelTasks::ETaskPriority Priority, CallableT&& InCallable) : Callable(Forward<CallableT>(InCallable))
		{
			Task.Init(DebugName, Priority, [this, Deleter(LowLevelTasks::TDeleter<ThisType, &ThisType::Finish>(this))]()
			{
				Execute();
			});
		}

		inline bool TryLaunch()
		{
			return LowLevelTasks::TryLaunch(Task);
		}

		inline bool IsLaunched() const
		{ 
			return !Task.IsReady();
		}

		inline void IncrementRefCount()
		{
			ReferenceCounter.fetch_add(1, std::memory_order_release);
		}

		inline bool IsCompleted() const
		{
			return Completed.IsNotified();
		}

		inline void Finish()
		{
			int LocalCounter = ReferenceCounter.fetch_sub(1, std::memory_order_release);
			if (LocalCounter == 2 && !IsLaunched()) //this is 2 because fetch_sub returns the value before the decrement (value is actually 1)
			{
				//Cancel the Task and Try to Launch the Continuation if we are the last reference
				verify(Task.TryCancel());
			}
			else if (LocalCounter == 1) //this is 1 because fetch_sub returns the value before the decrement (value is actually 0)
			{
				delete this;
			}
		}

		inline void GetResult()
		{
			if (Task.TryCancel())
			{
				Execute();
			}
			else
			{
				Completed.Wait();
			}
		}
	};
}

/*
 * Awaitable Tasks are very simple they only allow for Lauching and Awaiting a Task.
 * They will try to run other work while awaiting the Result.
 */
template<typename ReturnType>
class TAwaitableTask final
{
	template<typename R>
	using TPromiseVTableBase = AwaitableTask_Detail::TPromiseVTableBase<R>;

	template<typename R>
	using TPromiseVTableDummy = AwaitableTask_Detail::TPromiseVTableDummy<R>;

	template<typename R, typename C>
	using TPromise = AwaitableTask_Detail::TPromise<R, C>;

	template<typename P>
	using TPromiseVTable = AwaitableTask_Detail::TPromiseVTable<P>;

	using ThisClass = TAwaitableTask<ReturnType>;
	//the PromiseVTable has enough storage for both the promise and vtable
	TPromiseVTableBase<ReturnType> PromiseVTable;

public:
	TAwaitableTask()
	{
		//uninitizialized Handles start with a DummyVtable
		new (&PromiseVTable) TPromiseVTableDummy<ReturnType>();
	}

	~TAwaitableTask()
	{
		Reset();
	}

	TAwaitableTask(const ThisClass& Other)
	{
		PromiseVTable = Other.PromiseVTable;
		GetVtable()->IncrementRefCount();
	}

	TAwaitableTask(ThisClass&& Other)
	{
		PromiseVTable = MoveTemp(Other.PromiseVTable);
	}

	ThisClass& operator= (const ThisClass& Other)
	{
		if(this != &Other)
		{
			Reset();
			PromiseVTable = Other.PromiseVTable;
			GetVtable()->IncrementRefCount();
		}
		return *this;
	}

	ThisClass& operator= (ThisClass&& Other)
	{
		if(this != &Other)
		{
			Reset();
			PromiseVTable = MoveTemp(Other.PromiseVTable);
		}
		return *this;
	}

public:
	//initialize a new Task with given Priority now and Abandon previous work (if any)
	//initialized Tasks can be awaited on, but they will run synchronous in this case
	template<typename CallableT>
	void Init(const TCHAR* DebugName, LowLevelTasks::ETaskPriority Priority, CallableT&& Callable)
	{
		Reset();
		using CallableType = std::decay_t<CallableT>;
		static_assert(TIsInvocable<CallableType>::Value, "Callable is not invocable");
		static_assert(std::is_convertible<ReturnType, decltype(Callable())>::value,  "Callable has no matching Return Type");
		using PromiseType = TPromise<ReturnType, CallableType>;
		new (&PromiseVTable) TPromiseVTable<PromiseType>(new PromiseType(DebugName, Priority, Forward<CallableT>(Callable)));
	}

	//initialize a new Task now and Abandon previous work (if any)
	//initialized Tasks can be awaited on, but they will run synchronous in this case
	template<typename CallableT>
	inline void Init(const TCHAR* DebugName, CallableT&& Callable)
	{
		Init(DebugName, LowLevelTasks::ETaskPriority::Default, Forward<CallableT>(Callable));
	}

	//Launches the Task for processing on a WorkerThread. The returntype will specify if the Task was Launched successfully (true) or if it might have been launched already (false) 
	bool TryLaunch()
	{
		return GetVtable()->TryLaunch();
	}

	//initialize and Launch a Task with given Priority
	template<typename CallableT>
	bool InitAndLaunch(const TCHAR* DebugName, LowLevelTasks::ETaskPriority Priority, CallableT&& Callable)
	{
		Init(DebugName, Priority, Forward<CallableT>(Callable));
		using CallableType = std::decay_t<CallableT>;
		using PromiseVTableType = TPromiseVTable<TPromise<ReturnType, CallableType>>;
		return GetVtable<PromiseVTableType>()->TryLaunch();
	}

	//initialize and Launch a Task
	template<typename CallableT>
	inline bool InitAndLaunch(const TCHAR* DebugName, CallableT&& Callable)
	{
		return InitAndLaunch(DebugName, LowLevelTasks::ETaskPriority::Default, Forward<CallableT>(Callable));
	}

	//Abandon the Task and freeing its memory if this is the last reference to it.
	void Reset()
	{
		GetVtable()->Finish();
	}

	//returns true if the Task has been Launched.
	bool IsLaunched() const
	{
		return GetVtable()->IsLaunched();
	}

	//Does the TaskHandle hold a valid or Dummy Task?
	bool IsValid() const
	{
		return GetVtable()->IsValid();
	}

	//Returns true if the Tasks completed.
	bool IsCompleted() const
	{
		return GetVtable()->IsCompleted();
	}

	//Await the Task Completion and get the result, this will assert if this is a Dummy Handle.
	//Awaiting an initizialized but not launched task will run ('launch') it synchronously in the current thread.
	//use BusyWaitUntilLaunched if you want to wait for launching the task on another thread.
	ReturnType Await()
	{
		checkSlow(IsValid());
		return GetVtable()->GetResult();
	}

private:
	template<typename PromiseType = TPromiseVTableBase<ReturnType>>
	inline PromiseType* GetVtable()
	{
		return UE_LAUNDER(static_cast<PromiseType*>(&PromiseVTable));
	}

	template<typename PromiseType = TPromiseVTableBase<ReturnType>>
	inline const PromiseType* GetVtable() const
	{
		return UE_LAUNDER(static_cast<const PromiseType*>(&PromiseVTable));
	}
};
