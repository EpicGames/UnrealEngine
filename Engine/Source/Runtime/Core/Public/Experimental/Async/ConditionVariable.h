// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/ParkingLot.h"

namespace UE
{

/**
 * A one-byte portable condition variable. Gives the same decent performance everywhere.
 */
class FConditionVariable final
{
public:
    constexpr FConditionVariable() = default;

    FConditionVariable(const FConditionVariable&) = delete;
    FConditionVariable& operator=(const FConditionVariable&) = delete;

    void NotifyOne()
    {
        if (bMayHaveWaiters)
        {
            ParkingLot::WakeOne(
                &bMayHaveWaiters,
                [this] (ParkingLot::FWakeState WakeState) -> uint64
                {
                    if (!WakeState.bHasWaitingThreads)
                        bMayHaveWaiters = false;
                    return 0;
                });
        }
    }
    
    void NotifyAll()
    {
        if (bMayHaveWaiters)
        {
            bMayHaveWaiters = false;
            ParkingLot::WakeAll(&bMayHaveWaiters);
        }
    }

    template<typename TLock>
    void Wait(TLock& Lock)
    {
        ParkingLot::Wait(
            &bMayHaveWaiters,
            [this] () -> bool
            {
                bMayHaveWaiters = true;
                return true;
            },
            [&Lock] ()
            {
                Lock.Unlock();
            });
        Lock.Lock();
    }
    
	template<typename TLock>
	bool WaitFor(TLock& Lock, FMonotonicTimeSpan WaitTime)
    {
    	ParkingLot::FWaitState WaitState = ParkingLot::WaitFor(
			&bMayHaveWaiters,
			[this] () -> bool
			{
				bMayHaveWaiters = true;
				return true;
			},
			[&Lock] ()
			{
				Lock.Unlock();
			},
			WaitTime);
    	Lock.Lock();
     	return WaitState.bDidWake;
	}

	template<typename TLock>
	bool WaitUntil(TLock& Lock, FMonotonicTimePoint WaitTime)
    {
    	ParkingLot::FWaitState WaitState = ParkingLot::WaitUntil(
			&bMayHaveWaiters,
			[this] () -> bool
			{
				bMayHaveWaiters = true;
				return true;
			},
			[&Lock] ()
			{
				Lock.Unlock();
			},
			WaitTime);
    	Lock.Lock();
    	return WaitState.bDidWake;
    }

private:
    std::atomic<bool> bMayHaveWaiters = false;
};

} // namespace UE


