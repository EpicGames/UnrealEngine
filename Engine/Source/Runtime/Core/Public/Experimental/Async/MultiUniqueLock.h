// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Sort.h"
#include "Containers/Array.h"

namespace UE
{

// Like TUniqueLock, but acquires multiple locks and holds then until destruction. Acquires the locks in address
// order to prevent deadlocks if multiple threads are using TMultiUniqueLock for similar sets of locks.
template<typename TLockType>
class TMultiUniqueLock final
{
public:
    explicit TMultiUniqueLock(const TArray<TLockType*>& Locks)
        : Locks(Locks)
    {
        SortAndLock();
    }

    explicit TMultiUniqueLock(TArray<TLockType*>&& Locks)
        : Locks(MoveTemp(Locks))
    {
        SortAndLock();
    }

    TMultiUniqueLock& operator=(const TArray<TLockType*>&) = delete;
    TMultiUniqueLock& operator=(TArray<TLockType*>&&) = delete;

    ~TMultiUniqueLock()
    {
        for (TLockType* Lock : Locks)
        {
            Lock->Unlock();
        }
    }

private:
    void SortAndLock()
    {
        Algo::Sort(Locks);
        for (TLockType* Lock : Locks)
        {
            Lock->Lock();
        }
    }

    TArray<TLockType*> Locks;
};

}

