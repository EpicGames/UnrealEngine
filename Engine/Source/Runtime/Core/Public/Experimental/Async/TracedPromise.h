// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Async/Future.h"
#include "ProfilingDebugging/MiscTrace.h"

/* TPromise wrapper that creates a trace region */
template <typename ResultType>
class TTracedPromise
{
public:
	TTracedPromise(const TCHAR* RegionName, const TCHAR* RegionCategory = nullptr)
	{
#if MISCTRACE_ENABLED
		RegionId = TRACE_BEGIN_REGION_WITH_ID(RegionName, RegionCategory);
#endif
	}

	TTracedPromise(TUniqueFunction<void()>&& CompletionCallback, const TCHAR* RegionName, const TCHAR* RegionCategory = nullptr)
		: Promise(MoveTemp(CompletionCallback))
	{
#if MISCTRACE_ENABLED
		RegionId = TRACE_BEGIN_REGION_WITH_ID(RegionName, RegionCategory);
#endif
	}

	// Movable-only
	TTracedPromise(const TTracedPromise&) = delete;
	TTracedPromise& operator=(const TTracedPromise&) = delete;

	TTracedPromise(TTracedPromise&& Other)
		: Promise(MoveTemp(Other.Promise))
#if MISCTRACE_ENABLED
		, RegionId(Other.RegionId)
#endif
	{
#if MISCTRACE_ENABLED
		Other.RegionId = 0;
#endif
	}

	TTracedPromise& operator=(TTracedPromise&& Other)
	{
		if (this != &Other)
		{
			EndRegion();

			Promise = MoveTemp(Other.Promise);
#if MISCTRACE_ENABLED
			RegionId = Other.RegionId;
			Other.RegionId = 0;
#endif
		}

		return *this;
	}

	~TTracedPromise()
	{
		EndRegion();
	}

	TFuture<ResultType> GetFuture()
	{
		return Promise.GetFuture();
	}

	void SetValue(const ResultType& Value)
		requires(!std::is_void_v<ResultType>)
	{
		Promise.SetValue(Value);
		EndRegion();
	}

	void SetValue(ResultType&& Value)
		requires(!std::is_void_v<ResultType> && !std::is_lvalue_reference_v<ResultType>)
	{
		Promise.SetValue(MoveTemp(Value));
		EndRegion();
	}

	void SetValue()
		requires(std::is_void_v<ResultType>)
	{
		Promise.SetValue();
		EndRegion();
	}

private:
	void EndRegion()
	{
#if MISCTRACE_ENABLED
		if (RegionId != 0)
		{
			TRACE_END_REGION_WITH_ID(RegionId);
			RegionId = 0;
		}
#endif
	}

	TPromise<ResultType> Promise;
#if MISCTRACE_ENABLED
	/** Id of the traced timing region */
	uint64 RegionId = 0;
#endif
};
