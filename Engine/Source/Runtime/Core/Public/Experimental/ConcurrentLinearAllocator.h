// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>
#include <type_traits>

#include "AutoRTFM.h"
#include "HAL/Memory.h"
#include "HAL/UnrealMemory.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/MallocBinnedCommon.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTypeTraits.h"
#include "Containers/LockFreeFixedSizeAllocator.h"
#include "Misc/MemStack.h"

#if PLATFORM_HAS_ASAN_INCLUDE
#	include <sanitizer/asan_interface.h>
#	if defined(__SANITIZE_ADDRESS__)
#		define IS_ASAN_ENABLED 1
#	elif defined(__has_feature)
#		if __has_feature(address_sanitizer)
#			define IS_ASAN_ENABLED 1
#		else
#			define IS_ASAN_ENABLED 0
#		endif
#	else
#		define IS_ASAN_ENABLED 0
#	endif
#else
#	define ASAN_POISON_MEMORY_REGION(addr, size) ((void)(addr), (void)(size))
#	define ASAN_UNPOISON_MEMORY_REGION(addr, size) ((void)(addr), (void)(size))
#	define IS_ASAN_ENABLED 0
#endif


struct FAlignedAllocator
{
	static constexpr bool SupportsAlignment = true;
	static constexpr bool UsesFMalloc       = true;
	static constexpr uint32 MaxAlignment	= UE_MBC_MAX_SMALL_POOL_ALIGNMENT;

	inline static void* Malloc(SIZE_T Size, uint32 Alignment)
	{
		if (UNLIKELY(!FMEMORY_INLINE_GMalloc))
		{
			// There is no public function to create GMalloc and this will do it for us.
			FMemory::Free(FMemory::Malloc(0));
			check(FMEMORY_INLINE_GMalloc);
		}
		return FMEMORY_INLINE_GMalloc->Malloc(Size, Alignment);
	}

	UE_FORCEINLINE_HINT static void Free(void* Pointer, SIZE_T Size)
	{
		FMEMORY_INLINE_GMalloc->Free(Pointer);
	}
};

template<uint32 BlockSize, typename Allocator = FAlignedAllocator>
class TBlockAllocationCache
{
	struct FTLSCleanup
	{
		void* Block = nullptr;

		~FTLSCleanup()
		{
			if(Block)
			{
				Allocator::Free(Block, BlockSize);
			}
		}
	};

	inline static void* SwapBlock(void* NewBlock)
	{
		UE_MODULE_SAFE_THREAD_LOCAL(FTLSCleanup, Tls);
		void* Ret = Tls.Block;
		Tls.Block = NewBlock;
		return Ret;
	}

public:
	static constexpr bool SupportsAlignment = Allocator::SupportsAlignment;
	static constexpr bool UsesFMalloc       = Allocator::UsesFMalloc;
	static constexpr uint32 MaxAlignment	= Allocator::MaxAlignment;

	inline static void* Malloc(SIZE_T Size, uint32 Alignment)
	{
		if (Size == BlockSize)
		{
			void* Pointer = SwapBlock(nullptr);
			if (Pointer != nullptr)
			{
				return Pointer;
			}
		}
		return Allocator::Malloc(Size, Alignment);
	}

	inline static void Free(void* Pointer, SIZE_T Size)
	{
		if (Size == BlockSize)
		{
			Pointer = SwapBlock(Pointer);
			if (Pointer == nullptr)
			{
				return;
			}
		}
		return Allocator::Free(Pointer, Size);
	}
};

template <uint32 BlockSize, typename Allocator = FAlignedAllocator>
class TBlockAllocationLockFreeCache
{
public:
	static_assert(BlockSize == FPageAllocator::PageSize, "Only 64k pages are supported with this cache.");
	static constexpr bool SupportsAlignment = Allocator::SupportsAlignment;
	static constexpr bool UsesFMalloc       = Allocator::UsesFMalloc;
	static constexpr uint32 MaxAlignment	= Allocator::MaxAlignment;

	inline static void* Malloc(SIZE_T Size, uint32 Alignment)
	{
		if (Size == BlockSize)
		{
			return FPageAllocator::Get().Alloc(Alignment);
		}
		else
		{
			return Allocator::Malloc(Size, Alignment);
		}
	}

	inline static void Free(void* Pointer, SIZE_T Size)
	{
		if (Size == BlockSize)
		{
			FPageAllocator::Get().Free(Pointer);
		}
		else
		{
			Allocator::Free(Pointer, Size);
		}
	}
};

struct FDefaultBlockAllocationTag 
{
	static constexpr uint32 BlockSize = 64 * 1024;		// Blocksize used to allocate from
	static constexpr bool AllowOversizedBlocks = true;  // The allocator supports oversized Blocks and will store them in a seperate Block with counter 1
	static constexpr bool RequiresAccurateSize = true;  // GetAllocationSize returning the accurate size of the allocation otherwise it could be relaxed to return the size to the end of the Block
	static constexpr bool InlineBlockAllocation = false;  // Inline or Noinline the BlockAllocation which can have an impact on Performance
	static constexpr const char* TagName = "DefaultLinear";

	using Allocator = TBlockAllocationLockFreeCache<BlockSize, FAlignedAllocator>;
};

/*************************************************************************************************************************
 * This fast linear allocator can be used for temporary allocations, and is best suited for allocations that are 
 * produced and consumed on different threads and within the lifetime of a frame. Although the lifetime of any 
 * individual allocation is not hard-tied to a frame (tracking is done using the FBlockHeader::NumAllocations atomic 
 * variable), the application will eventually run OOM if allocations are not cleaned up in a timely manner.
 * 
 * There is a fast-path version of the allocator that skips AllocationHeaders by aligning the BlockHeader with the 
 * BlockSize, so that headers can easily be found by AligningDown the address of the Allocation itself. 
 * The allocator works by allocating a larger block in TLS which has a Header at the front which contains the atomic,
 * and all allocations are than allocated from this block:
 *
 * ------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------
 * | FBlockHeader(atomic counter etc.) | Alignment Waste | FAllocationHeader(size, optional) | Memory used for Allocation | Alignment Waste | FAllocationHeader(size, optional) | Memory used for Allocation | FreeSpace ... 
 * ------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------
 *
 * The allocator is most often used concurrently, but also supports single-threaded use cases, so it can be used for an
 * array scratchpad.
 */
enum class ELinearAllocatorThreadPolicy
{
	ThreadSafe,
	NotThreadSafe
};

template <typename BlockAllocationTag, ELinearAllocatorThreadPolicy ThreadPolicy>
class TLinearAllocatorBase
{
	static constexpr bool SupportsFastPath = ((BlockAllocationTag::BlockSize <= (64 * 1024)) && (FPlatformProperties::GetMaxSupportedVirtualMemoryAlignment() >= 64 * 1024))
		&& FMath::IsPowerOfTwo(BlockAllocationTag::BlockSize) // AlignDown only works with Pow2
		&& !IS_ASAN_ENABLED && !BlockAllocationTag::RequiresAccurateSize // Only enabled when not using ASAN or when the Size of an allocation is not required 
		&& BlockAllocationTag::Allocator::SupportsAlignment; // The allocator needs to align the BlockHeader by BlockSize to find the BlockHeader

	LLM(static constexpr ELLMAllocType LLMAllocationType = BlockAllocationTag::Allocator::UsesFMalloc ? ELLMAllocType::FMalloc : ELLMAllocType::None;)

	struct FBlockHeader;

	class FAllocationHeader
	{
	public:
		inline FAllocationHeader(FBlockHeader* InBlockHeader, SIZE_T InAllocationSize) 
		{
			uintptr_t Offset = uintptr_t(this) - uintptr_t(InBlockHeader);
			checkSlow(Offset < UINT32_MAX);
			BlockHeaderOffset = uint32(Offset);

			checkSlow(InAllocationSize < UINT32_MAX);
			AllocationSize = uint32(InAllocationSize);
		}

		UE_FORCEINLINE_HINT FBlockHeader* GetBlockHeader() const
		{
			return reinterpret_cast<FBlockHeader*>(uintptr_t(this) - BlockHeaderOffset);
		}

		UE_FORCEINLINE_HINT SIZE_T GetAllocationSize() const
		{
			return SIZE_T(AllocationSize);
		}

	private:
		uint32 BlockHeaderOffset; //this is the negative offset from the allocation to the BlockHeader.
		uint32 AllocationSize; //this is the size of the allocation following the AllocationHeader.
	};

	static inline FAllocationHeader* GetAllocationHeader(void* Pointer)
	{
		if (SupportsFastPath)
		{
			return nullptr;
		}
		else
		{
			return reinterpret_cast<FAllocationHeader*>(Pointer) - 1;
		}
	}

	struct FBlockHeader
	{
		UE_FORCEINLINE_HINT FBlockHeader() 
			//We need to reserve space for at least one BlockHeader as well as one AllocationHeader
			: NextAllocationPtr(uintptr_t(this) + sizeof(FBlockHeader) + sizeof(FAllocationHeader))
		{
		}

		struct FPlainUInt
		{
			/** Returns the original number of allocations, before subtraction occurs. */
			unsigned int FetchSub(unsigned int N, std::memory_order)
			{
				unsigned int OriginalValue = Value;
				Value -= N;
				return OriginalValue;
			}

			void Store(unsigned int N, std::memory_order)
			{
				Value = N;
			}

			unsigned int Value;
		};

		struct FAtomicUInt
		{
			/** Returns the original number of allocations, before subtraction occurs. */
			unsigned int FetchSub(unsigned int N, std::memory_order Order)
			{
				return Value.fetch_sub(N, Order);
			}

			void Store(unsigned int N, std::memory_order Order)
			{
				return Value.store(N, Order);
			}

			std::atomic_uint Value;
		};

		using NumAllocationsType = std::conditional_t<ThreadPolicy == ELinearAllocatorThreadPolicy::ThreadSafe, FAtomicUInt, FPlainUInt>;

		NumAllocationsType NumAllocations { .Value = UINT_MAX }; //this is shared between threads and tracks the number of live allocations (plus UINT_MAX) and we will fix this up when we close a block.
		uint8 Padding[PLATFORM_CACHE_LINE_SIZE - sizeof(std::atomic_uint)]; //avoid false sharing
		uintptr_t NextAllocationPtr; //this is the next address we are trying to allocate from.
		unsigned int Num = 0; //this tracks the TLS local Number of allocation from a given Block
	};

	struct FTLSCleanup
	{
		FBlockHeader* Header = nullptr;

		~FTLSCleanup()
		{
			if(Header)
			{
				Header->NextAllocationPtr = uintptr_t(Header) + BlockAllocationTag::BlockSize;
				const uint32 DeltaCount = UINT_MAX - Header->Num; 

				// on the allocating side we only need to do a single atomic to reduce contention with the deletions
				// this will leave the atomic in a state where it only counts the number of live allocations (before it was based of UINT_MAX)
				if (Header->NumAllocations.FetchSub(DeltaCount, std::memory_order_acq_rel) == DeltaCount)
				{
					//if all allocations are already freed we can reuse the Block again
					Header->~FBlockHeader();
					ASAN_UNPOISON_MEMORY_REGION( Header, BlockAllocationTag::BlockSize );
					MemoryTrace_UnmarkAllocAsHeap(uint64(Header), EMemoryTraceRootHeap::SystemMemory);
					BlockAllocationTag::Allocator::Free(Header, BlockAllocationTag::BlockSize);
				}
			}
		}
	};

	FORCENOINLINE static void AllocateBlock(FBlockHeader*& Header)
	{
		if constexpr (!BlockAllocationTag::InlineBlockAllocation)
		{
			static_assert(BlockAllocationTag::BlockSize >= sizeof(FBlockHeader) + sizeof(FAllocationHeader));
			uint32 BlockAlignment = SupportsFastPath ? BlockAllocationTag::BlockSize : alignof(FBlockHeader);
			Header = new (BlockAllocationTag::Allocator::Malloc(BlockAllocationTag::BlockSize, BlockAlignment)) FBlockHeader;
			MemoryTrace_MarkAllocAsHeap(uint64(Header), EMemoryTraceRootHeap::SystemMemory);
			checkSlow(IsAligned(Header, BlockAlignment));
			if constexpr (!SupportsFastPath)
			{
				ASAN_POISON_MEMORY_REGION( Header + 1, BlockAllocationTag::BlockSize - sizeof(FBlockHeader) );
			}

			UE_MODULE_SAFE_THREAD_LOCAL(FTLSCleanup, Cleanup);
			new (&Cleanup) FTLSCleanup{ Header };
		}
	}

public:
	template<uint32 Alignment> //this can help the compiler folding away the alignment when it is statically known
	static UE_FORCEINLINE_HINT void* Malloc(SIZE_T Size)
	{
		return Malloc(Size, Alignment);
	}

	template<typename T>
	static UE_FORCEINLINE_HINT void* Malloc()
	{
		return Malloc(sizeof(T), alignof(T));
	}

	static inline void* Malloc(SIZE_T Size, uint32 Alignment)
	{
		checkSlow(Alignment >= 1 && FMath::IsPowerOfTwo(Alignment));

		if (AutoRTFM::IsClosed())
		{
			void* const Pointer = AutoRTFM::Open([Size, Alignment]
			{
				return Malloc(Size, Alignment);
			});

			AutoRTFM::OnAbort([Pointer]
			{
				Free(Pointer);
			});

			return Pointer;
		}

		if constexpr (!SupportsFastPath)
		{
			Alignment = (Alignment < alignof(FAllocationHeader)) ? alignof(FAllocationHeader) : Alignment; 
#if IS_ASAN_ENABLED
			// Make sure FAllocationHeader is 8 bytes aligned so poison / unpoison doesnt end up as false positive
			Alignment = Align(Alignment, 8);
#endif
		}

		UE_MODULE_SAFE_THREAD_LOCAL(FBlockHeader*, Header);
		if (Header == nullptr)
		{
		AllocateNewBlock:
			if constexpr (BlockAllocationTag::InlineBlockAllocation)
			{
				static_assert(BlockAllocationTag::BlockSize >= sizeof(FBlockHeader) + sizeof(FAllocationHeader));
				uint32 BlockAlignment = SupportsFastPath ? BlockAllocationTag::BlockSize : alignof(FBlockHeader);
				Header = new (BlockAllocationTag::Allocator::Malloc(BlockAllocationTag::BlockSize, BlockAlignment)) FBlockHeader;
				MemoryTrace_MarkAllocAsHeap(uint64(Header), EMemoryTraceRootHeap::SystemMemory);
				checkSlow(IsAligned(Header, BlockAlignment));
				if constexpr (!SupportsFastPath)
				{
					ASAN_POISON_MEMORY_REGION( Header + 1, BlockAllocationTag::BlockSize - sizeof(FBlockHeader) );
				}

				UE_MODULE_SAFE_THREAD_LOCAL(FTLSCleanup, Cleanup);
				new (&Cleanup) FTLSCleanup{ Header };
			}
			else
			{
				AllocateBlock(Header);
			}
		}

	AllocateNewItem:
		if constexpr (SupportsFastPath)
		{
			uintptr_t AlignedOffset = Align(Header->NextAllocationPtr, Alignment);
			if (AlignedOffset + Size <= uintptr_t(Header) + BlockAllocationTag::BlockSize)
			{
				Header->NextAllocationPtr = AlignedOffset + Size;
				Header->Num++;

				MemoryTrace_Alloc(uint64(AlignedOffset), Size, Alignment);
				LLM_IF_ENABLED( FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, reinterpret_cast<void*>(AlignedOffset), Size, ELLMTag::LinearAllocator, LLMAllocationType) );
				return reinterpret_cast<void*>(AlignedOffset);
			}

			// The cold path of the code starts here
			constexpr SIZE_T HeaderSize = sizeof(FBlockHeader);
			if constexpr (BlockAllocationTag::AllowOversizedBlocks)
			{
				//support for oversized Blocks
				if (HeaderSize + Size + Alignment > BlockAllocationTag::BlockSize)
				{
					FBlockHeader* LargeHeader = new (BlockAllocationTag::Allocator::Malloc(HeaderSize + Size + Alignment, BlockAllocationTag::BlockSize)) FBlockHeader;
					MemoryTrace_MarkAllocAsHeap(uint64(LargeHeader), EMemoryTraceRootHeap::SystemMemory);
					checkSlow(IsAligned(LargeHeader, alignof(FBlockHeader)));

					uintptr_t LargeAlignedOffset = Align(LargeHeader->NextAllocationPtr, Alignment);
					LargeHeader->NextAllocationPtr = uintptr_t(LargeHeader) + HeaderSize + Size + Alignment;
					LargeHeader->NumAllocations.Store(1, std::memory_order_release);

					checkSlow(LargeAlignedOffset + Size <= LargeHeader->NextAllocationPtr);

					MemoryTrace_Alloc(uint64(LargeAlignedOffset), Size, Alignment);
					LLM_IF_ENABLED( FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, reinterpret_cast<void*>(LargeAlignedOffset), Size, ELLMTag::LinearAllocator, LLMAllocationType) );
					return reinterpret_cast<void*>(LargeAlignedOffset);
				}
			}
			check(HeaderSize + Size + Alignment <= BlockAllocationTag::BlockSize);
		}
		else
		{
			uintptr_t AlignedOffset = Align(Header->NextAllocationPtr, Alignment);
			if (AlignedOffset + Size <= uintptr_t(Header) + BlockAllocationTag::BlockSize)
			{
				Header->NextAllocationPtr = AlignedOffset + Size + sizeof(FAllocationHeader);
				Header->Num++;

				FAllocationHeader* AllocationHeader = reinterpret_cast<FAllocationHeader*>(AlignedOffset) - 1;
				ASAN_UNPOISON_MEMORY_REGION( AllocationHeader, sizeof(FAllocationHeader) + Size );
				new (AllocationHeader) FAllocationHeader(Header, Size);
				ASAN_POISON_MEMORY_REGION( AllocationHeader, sizeof(FAllocationHeader) );

				MemoryTrace_Alloc(uint64(AllocationHeader), Size + sizeof(FAllocationHeader), Alignment);
				LLM_IF_ENABLED( FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, AllocationHeader, Size + sizeof(FAllocationHeader), ELLMTag::LinearAllocator, LLMAllocationType) );
				return reinterpret_cast<void*>(AlignedOffset);
			}

			// The cold path of the code starts here
			constexpr SIZE_T HeaderSize = sizeof(FBlockHeader) + sizeof(FAllocationHeader);
			if constexpr (BlockAllocationTag::AllowOversizedBlocks)
			{
				//support for oversized Blocks
				if (HeaderSize + Size + Alignment > BlockAllocationTag::BlockSize)
				{
					FBlockHeader* LargeHeader = new (BlockAllocationTag::Allocator::Malloc(HeaderSize + Size + Alignment, alignof(FBlockHeader))) FBlockHeader;
					MemoryTrace_MarkAllocAsHeap(uint64(LargeHeader), EMemoryTraceRootHeap::SystemMemory);
					checkSlow(IsAligned(LargeHeader, alignof(FBlockHeader)));

					uintptr_t LargeAlignedOffset = Align(LargeHeader->NextAllocationPtr, Alignment);
					LargeHeader->NextAllocationPtr = uintptr_t(LargeHeader) + HeaderSize + Size + Alignment;
					LargeHeader->NumAllocations.Store(1, std::memory_order_release);

					checkSlow(LargeAlignedOffset + Size <= LargeHeader->NextAllocationPtr);
					FAllocationHeader* AllocationHeader = new (reinterpret_cast<FAllocationHeader*>(LargeAlignedOffset) - 1) FAllocationHeader(LargeHeader, Size);
					ASAN_POISON_MEMORY_REGION( AllocationHeader, sizeof(FAllocationHeader) );

					MemoryTrace_Alloc(uint64(AllocationHeader), Size + sizeof(FAllocationHeader), Alignment);
					LLM_IF_ENABLED( FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, AllocationHeader, Size + sizeof(FAllocationHeader), ELLMTag::LinearAllocator, LLMAllocationType) );
					return reinterpret_cast<void*>(LargeAlignedOffset);
				}
			}
			check(HeaderSize + Size + Alignment <= BlockAllocationTag::BlockSize);
		}

		//allocation of a new Block
		Header->NextAllocationPtr = uintptr_t(Header) + BlockAllocationTag::BlockSize;
		const uint32 DeltaCount = UINT_MAX - Header->Num; 

		// on the allocating side we only need to do a single atomic to reduce contention with the deletions
		// this will leave the atomic in a state where it only counts the number of live allocations (before it was based of UINT_MAX)
		if (Header->NumAllocations.FetchSub(DeltaCount, std::memory_order_acq_rel) == DeltaCount)
		{
			//if all allocations are already freed we can reuse the Block again
			Header->~FBlockHeader();
			Header = new (Header) FBlockHeader;
			ASAN_POISON_MEMORY_REGION( Header + 1, BlockAllocationTag::BlockSize - sizeof(FBlockHeader) );
			goto AllocateNewItem;
		}

		goto AllocateNewBlock;
	}

	static inline void Free(void* Pointer)
	{
		if(Pointer != nullptr)
		{
			if (AutoRTFM::IsClosed())
			{
				AutoRTFM::OnCommit([Pointer]
				{
					Free(Pointer);
				});

				return;
			}

			if constexpr (SupportsFastPath)
			{
				MemoryTrace_Free(uint64(Pointer));
				LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, Pointer, LLMAllocationType));
				FBlockHeader* Header = reinterpret_cast<FBlockHeader*>(AlignDown(Pointer, BlockAllocationTag::BlockSize));

				// this deletes complete blocks when the last allocation is freed
				if (Header->NumAllocations.FetchSub(1, std::memory_order_acq_rel) == 1)
				{
					const uintptr_t NextAllocationPtr = Header->NextAllocationPtr;
					Header->~FBlockHeader();
					MemoryTrace_UnmarkAllocAsHeap(uint64(Header), EMemoryTraceRootHeap::SystemMemory);
					BlockAllocationTag::Allocator::Free(Header, NextAllocationPtr - uintptr_t(Header));
				}
			}
			else
			{
				FAllocationHeader* AllocationHeader = GetAllocationHeader(Pointer);
				ASAN_UNPOISON_MEMORY_REGION( AllocationHeader, sizeof(FAllocationHeader) );
				MemoryTrace_Free(uint64(AllocationHeader));
				LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, AllocationHeader, LLMAllocationType));
				FBlockHeader* Header = AllocationHeader->GetBlockHeader();
				ASAN_POISON_MEMORY_REGION( AllocationHeader, sizeof(FAllocationHeader) + AllocationHeader->GetAllocationSize() );

				// this deletes complete blocks when the last allocation is freed
				if (Header->NumAllocations.FetchSub(1, std::memory_order_acq_rel) == 1)
				{
					const uintptr_t NextAllocationPtr = Header->NextAllocationPtr;
					Header->~FBlockHeader();
					ASAN_UNPOISON_MEMORY_REGION( Header, NextAllocationPtr - uintptr_t(Header) );
					MemoryTrace_UnmarkAllocAsHeap(uint64(Header), EMemoryTraceRootHeap::SystemMemory);
					BlockAllocationTag::Allocator::Free(Header, NextAllocationPtr - uintptr_t(Header));
				}
			}
		}
	}

	AUTORTFM_OPEN static inline SIZE_T GetAllocationSize(void* Pointer)
	{
		if (Pointer)
		{
			if constexpr (SupportsFastPath)
			{
				return Align(uintptr_t(Pointer), BlockAllocationTag::BlockSize) - uintptr_t(Pointer);
			}
			else
			{
				FAllocationHeader* AllocationHeader = GetAllocationHeader(Pointer);
				ASAN_UNPOISON_MEMORY_REGION( AllocationHeader, sizeof(FAllocationHeader) );
				SIZE_T Size = AllocationHeader->GetAllocationSize();
				ASAN_POISON_MEMORY_REGION( AllocationHeader, sizeof(FAllocationHeader) );
				return Size;
			}
		}
		return 0;
	}

	static inline void* Realloc(void* Old, SIZE_T Size, uint32 Alignment)
	{
		void* New = nullptr;
		if(Size != 0)
		{
			New = Malloc(Size, Alignment);
			SIZE_T OldSize = GetAllocationSize(Old);
			if(OldSize != 0)
			{
				memcpy(New, Old, Size < OldSize ? Size : OldSize);
			}
		}
		Free(Old);
		return New;
	}
};

template <typename T>
using TConcurrentLinearAllocator = TLinearAllocatorBase<T, ELinearAllocatorThreadPolicy::ThreadSafe>;

using FConcurrentLinearAllocator = TLinearAllocatorBase<FDefaultBlockAllocationTag, ELinearAllocatorThreadPolicy::ThreadSafe>;
using FNonconcurrentLinearAllocator = TLinearAllocatorBase<FDefaultBlockAllocationTag, ELinearAllocatorThreadPolicy::NotThreadSafe>;

template<typename ObjectType, typename BlockAllocationTag = FDefaultBlockAllocationTag>
class TConcurrentLinearObject
{
public:
	inline static void* operator new(size_t Size)
	{
		static_assert(TIsDerivedFrom<ObjectType, TConcurrentLinearObject<ObjectType, BlockAllocationTag>>::Value, "TConcurrentLinearObject must be base of it's ObjectType (see CRTP)");
		static_assert(alignof(ObjectType) <= BlockAllocationTag::Allocator::MaxAlignment);
		return TConcurrentLinearAllocator<BlockAllocationTag>::template Malloc<alignof(ObjectType)>(Size);
	}

	inline static void* operator new(size_t Size, void* Object)
	{
		static_assert(TIsDerivedFrom<ObjectType, TConcurrentLinearObject<ObjectType, BlockAllocationTag>>::Value, "TConcurrentLinearObject must be base of it's ObjectType (see CRTP)");
		return Object;
	}

	inline static void* operator new[](size_t Size)
	{
		static_assert(alignof(ObjectType) <= BlockAllocationTag::Allocator::MaxAlignment);
		return TConcurrentLinearAllocator<BlockAllocationTag>::template Malloc<alignof(ObjectType)>(Size);
	}

	inline static void* operator new(size_t Size, std::align_val_t Align)
	{
		static_assert(alignof(ObjectType) <= BlockAllocationTag::Allocator::MaxAlignment);
		checkSlow(size_t(Align) == alignof(ObjectType));
		return TConcurrentLinearAllocator<BlockAllocationTag>::template Malloc<alignof(ObjectType)>(Size);
	}

	inline static void* operator new[](size_t Size, std::align_val_t Align)
	{
		static_assert(alignof(ObjectType) <= BlockAllocationTag::Allocator::MaxAlignment);
		checkSlow(size_t(Align)  == alignof(ObjectType));
		return TConcurrentLinearAllocator<BlockAllocationTag>::template Malloc<alignof(ObjectType)>(Size);
	}

	UE_FORCEINLINE_HINT static void operator delete(void* Ptr)
	{
		return TConcurrentLinearAllocator<BlockAllocationTag>::Free(Ptr);
	}

	UE_FORCEINLINE_HINT static  void operator delete[](void* Ptr)
	{
		return TConcurrentLinearAllocator<BlockAllocationTag>::Free(Ptr);
	}
};

namespace UE::Core::Private
{
	[[noreturn]] CORE_API void OnInvalidConcurrentLinearArrayAllocatorNum(int32 NewNum, SIZE_T NumBytesPerElement);
}

template <typename BlockAllocationTag, ELinearAllocatorThreadPolicy ThreadPolicy>
class TLinearArrayAllocatorBase
{
public:
	using SizeType = int32;

	enum { NeedsElementType = true };
	enum { RequireRangeCheck = true };

	template<typename ElementType>
	class ForElementType
	{
	public:
		ForElementType() = default;
		~ForElementType()
		{
			if (Data)
			{
				TLinearAllocatorBase<BlockAllocationTag, ThreadPolicy>::Free(Data);
			}
		}
		/**
		* Moves the state of another allocator into this one.
		* Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		* @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		*/
		inline void MoveToEmpty(ForElementType& Other)
		{
			checkSlow(this != &Other);

			if (Data)
			{
				TLinearAllocatorBase<BlockAllocationTag, ThreadPolicy>::Free(Data);
			}

			Data = Other.Data;
			Other.Data = nullptr;
		}
		inline ElementType* GetAllocation() const
		{
			return Data;
		}
		void ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement)
		{
			static_assert(sizeof(int32) <= sizeof(SIZE_T), "SIZE_T is expected to be larger than int32");

			// Check for under/overflow
			if (UNLIKELY(NewMax < 0 || NumBytesPerElement < 1 || NumBytesPerElement > (SIZE_T)MAX_int32))
			{
				UE::Core::Private::OnInvalidConcurrentLinearArrayAllocatorNum(NewMax, NumBytesPerElement);
			}

			static_assert(alignof(ElementType) <= BlockAllocationTag::Allocator::MaxAlignment);
			Data = (ElementType*)TLinearAllocatorBase<BlockAllocationTag, ThreadPolicy>::Realloc(Data, NewMax * NumBytesPerElement, alignof(ElementType));
		}
		SizeType CalculateSlackReserve(SizeType NewMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackReserve(NewMax, NumBytesPerElement, false);
		}
		SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, false);
		}
		SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackGrow(NewMax, CurrentMax, NumBytesPerElement, false);
		}

		SIZE_T GetAllocatedSize(SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return CurrentMax * NumBytesPerElement;
		}

		bool HasAllocation() const
		{
			return !!Data;
		}

		SizeType GetInitialCapacity() const
		{
			return 0;
		}


	private:
		ForElementType(const ForElementType&) = delete;
		ForElementType& operator=(const ForElementType&) = delete;
		ElementType* Data = nullptr;
	};

	using ForAnyElementType = ForElementType<FScriptContainerElement>;
};

template <typename BlockAllocationTag>
using TConcurrentLinearArrayAllocator = TLinearArrayAllocatorBase<BlockAllocationTag, ELinearAllocatorThreadPolicy::ThreadSafe>;

template <typename BlockAllocationTag>
using TNonconcurrentLinearArrayAllocator = TLinearArrayAllocatorBase<BlockAllocationTag, ELinearAllocatorThreadPolicy::NotThreadSafe>;

template <typename BlockAllocationTag>
struct TAllocatorTraits<TConcurrentLinearArrayAllocator<BlockAllocationTag>> : TAllocatorTraitsBase<TConcurrentLinearArrayAllocator<BlockAllocationTag>>
{
	enum { IsZeroConstruct = true };
};

template <typename BlockAllocationTag>
using TConcurrentLinearBitArrayAllocator = TInlineAllocator<4, TConcurrentLinearArrayAllocator<BlockAllocationTag>>;

template <typename BlockAllocationTag>
using TConcurrentLinearSparseArrayAllocator = TSparseArrayAllocator<TConcurrentLinearArrayAllocator<BlockAllocationTag>, TConcurrentLinearBitArrayAllocator<BlockAllocationTag>>;

template <typename BlockAllocationTag>
using TConcurrentLinearSetAllocator = TSetAllocator<TConcurrentLinearSparseArrayAllocator<BlockAllocationTag>, TInlineAllocator<1, TConcurrentLinearBitArrayAllocator<BlockAllocationTag>>>;

using FConcurrentLinearArrayAllocator = TConcurrentLinearArrayAllocator<FDefaultBlockAllocationTag>;
using FConcurrentLinearBitArrayAllocator = TConcurrentLinearBitArrayAllocator<FDefaultBlockAllocationTag>;
using FConcurrentLinearSparseArrayAllocator = TConcurrentLinearSparseArrayAllocator<FDefaultBlockAllocationTag>;
using FConcurrentLinearSetAllocator = TConcurrentLinearSetAllocator<FDefaultBlockAllocationTag>;

using FNonconcurrentLinearArrayAllocator = TNonconcurrentLinearArrayAllocator<FDefaultBlockAllocationTag>;

//The BulkObjectAllocator can be used to atomically destroy all allocated Objects, it will properly call every destructor before deleting the memory as well
template<typename BlockAllocationTag>
class TConcurrentLinearBulkObjectAllocator
{
	using ThisType = TConcurrentLinearBulkObjectAllocator<BlockAllocationTag>;
	struct FAllocation
	{
		virtual ~FAllocation() = default;
		FAllocation* Next = nullptr;
	};

	template <typename T>
	struct TObject final : FAllocation
	{
		TObject() = default;
		virtual ~TObject() override
		{
			T* Alloc = reinterpret_cast<T*>(uintptr_t(this) + Align(sizeof(TObject<T>), alignof(T)));
			checkSlow(IsAligned(Alloc, alignof(T)));

			// We need a typedef here because VC won't compile the destructor call below if ElementType itself has a member called ElementType
			using DestructorType = T;
			Alloc->DestructorType::~T();
		}
	};

	template <typename T>
	struct TObjectArray final : FAllocation
	{
		inline TObjectArray(SIZE_T InNum) 
			: Num(InNum) 
		{
		}

		virtual ~TObjectArray() override
		{
			T* Alloc = reinterpret_cast<T*>(uintptr_t(this) + Align(sizeof(TObjectArray<T>), alignof(T)));
			checkSlow(IsAligned(Alloc, alignof(T)));

			for (SIZE_T i = 0; i < Num; i++)
			{
				// We need a typedef here because VC won't compile the destructor call below if ElementType itself has a member called ElementType
				using DestructorType = T;
				Alloc[i].DestructorType::~T();
			}
		}
		SIZE_T Num;
	};

	std::atomic<FAllocation*> Next { nullptr };

public:
	TConcurrentLinearBulkObjectAllocator() = default;
	~TConcurrentLinearBulkObjectAllocator()
	{
		BulkDelete();
	}

	inline void BulkDelete()
	{
		FAllocation* Allocation = Next.exchange(nullptr, std::memory_order_acquire);
		while (Allocation != nullptr)
		{
			FAllocation* NextAllocation = Allocation->Next;
			Allocation->~FAllocation();
			TConcurrentLinearAllocator<BlockAllocationTag>::Free(Allocation);
			Allocation = NextAllocation;
		}
	}

	// In a transaction because the bulk allocator only has bulk delete, on abort we're just wasting the memory.
	// So we run this code in the open (non-transactionally) always.
	AUTORTFM_OPEN inline void* Malloc(SIZE_T Size, uint32 Alignment)
	{
		SIZE_T TotalSize = Align(sizeof(FAllocation), Alignment) + Size;
		FAllocation* Allocation = new (TConcurrentLinearAllocator<BlockAllocationTag>::Malloc(TotalSize, FMath::Max(static_cast<uint32>(alignof(FAllocation)), Alignment))) FAllocation();

		void* Alloc = Align(Allocation + 1, Alignment);
		checkSlow(uintptr_t(Alloc) + Size - uintptr_t(Allocation) <= TotalSize);

		Allocation->Next = Next.load(std::memory_order_relaxed);
		while (!Next.compare_exchange_strong(Allocation->Next, Allocation, std::memory_order_release, std::memory_order_relaxed))
		{
		}
		return Alloc;
	}

	inline void* MallocAndMemset(SIZE_T Size, uint32 Alignment, uint8 MemsetChar)
	{
		void* Ptr = Malloc(Size, Alignment);
		FMemory::Memset(Ptr, MemsetChar, Size);
		return Ptr;
	}

	template <typename T>
	inline T* Malloc()
	{
		return reinterpret_cast<T*>(Malloc(sizeof(T), alignof(T)));
	}

	template <typename T>
	inline T* MallocAndMemset(uint8 MemsetChar)
	{
		void* Ptr = Malloc(sizeof(T), alignof(T));
		FMemory::Memset(Ptr, MemsetChar, sizeof(T));
		return reinterpret_cast<T*>(Ptr);
	}

	template <typename T>
	inline T* MallocArray(SIZE_T Num)
	{
		return reinterpret_cast<T*>(Malloc(sizeof(T) * Num, alignof(T)));
	}

	template <typename T>
	inline T* MallocAndMemsetArray(SIZE_T Num, uint8 MemsetChar)
	{
		void* Ptr = Malloc(sizeof(T) * Num, alignof(T));
		FMemory::Memset(Ptr, MemsetChar, sizeof(T) * Num);
		return reinterpret_cast<T*>(Ptr);
	}

	template<typename T, typename... TArgs>
	inline T* Create(TArgs&&... Args)
	{
		T* Alloc = CreateNoInit<T>();
		new ((void*)Alloc) T(Forward<TArgs>(Args)...);
		return Alloc;
	}

	template<typename T, typename... TArgs>
	inline T* CreateArray(SIZE_T Num, const TArgs&... Args)
	{
		T* Alloc = CreateArrayNoInit<T>(Num);
		for (SIZE_T i = 0; i < Num; i++)
		{
			new ((void*)(&Alloc[i])) T(Args...);
		}
		return Alloc;
	}

private: //Kept private, there is also deliberately no new operator overloads because those will not be able to send the ObjectType into the allocator defeating its purpose 
	template<typename T>
	inline T* CreateNoInit()
	{
		SIZE_T TotalSize = Align(sizeof(TObject<T>), alignof(T)) + sizeof(T);
		TObject<T>* Object = new (TConcurrentLinearAllocator<BlockAllocationTag>::template Malloc<FMath::Max(alignof(TObject<T>), alignof(T))>(TotalSize)) TObject<T>();

		T* Alloc = reinterpret_cast<T*>(uintptr_t(Object) + Align(sizeof(TObject<T>), alignof(T)));
		checkSlow(IsAligned(Alloc, alignof(T)));
		checkSlow(uintptr_t(Alloc + 1) - uintptr_t(Object) <= TotalSize);

		Object->Next = Next.load(std::memory_order_relaxed);
		while (!Next.compare_exchange_weak(Object->Next, Object, std::memory_order_release, std::memory_order_relaxed))
		{
		}
		return Alloc;
	}

	template<typename T>
	inline T* CreateArrayNoInit(SIZE_T Num)
	{
		SIZE_T TotalSize = Align(sizeof(TObjectArray<T>), alignof(T)) + (sizeof(T) * Num);
		TObjectArray<T>* Array = new (TConcurrentLinearAllocator<BlockAllocationTag>::template Malloc<FMath::Max(alignof(TObjectArray<T>), alignof(T))>(TotalSize)) TObjectArray<T>(Num);

		T* Alloc = reinterpret_cast<T*>(uintptr_t(Array) + Align(sizeof(TObjectArray<T>), alignof(T)));
		checkSlow(IsAligned(Alloc, alignof(T)));
		checkSlow(uintptr_t(Alloc + Num) - uintptr_t(Array) <= TotalSize);

		Array->Next = Next.load(std::memory_order_relaxed);
		while (!Next.compare_exchange_weak(Array->Next, Array, std::memory_order_release, std::memory_order_relaxed))
		{
		}
		return Alloc;
	}
};

using FConcurrentLinearBulkObjectAllocator = TConcurrentLinearBulkObjectAllocator<FDefaultBlockAllocationTag>;
