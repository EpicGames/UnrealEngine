/******************************************************************************
 * Copyright (c) 2014-2016, Pedro Ramalhete, Andreia Correia
 * All rights reserved.
 *
 * Redistribution and use in source and binary forms, with or without
 * modification, are permitted provided that the following conditions are met:
 *     * Redistributions of source code must retain the above copyright
 *       notice, this list of conditions and the following disclaimer.
 *     * Redistributions in binary form must reproduce the above copyright
 *       notice, this list of conditions and the following disclaimer in the
 *       documentation and/or other materials provided with the distribution.
 *     * Neither the name of Concurrency Freaks nor the
 *       names of its contributors may be used to endorse or promote products
 *       derived from this software without specific prior written permission.
 *
 * THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS "AS IS" AND
 * ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT LIMITED TO, THE IMPLIED
 * WARRANTIES OF MERCHANTABILITY AND FITNESS FOR A PARTICULAR PURPOSE ARE
 * DISCLAIMED. IN NO EVENT SHALL <COPYRIGHT HOLDER> BE LIABLE FOR ANY
 * DIRECT, INDIRECT, INCIDENTAL, SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES
 * (INCLUDING, BUT NOT LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES;
 * LOSS OF USE, DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND
 * ON ANY THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
 * (INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE OF THIS
 * SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
 ******************************************************************************
 */

#ifndef _FAA_ARRAY_QUEUE_HP_H_
#define _FAA_ARRAY_QUEUE_HP_H_

#include <atomic>
#include "HazardPointer.h"
#include "HAL/PlatformProcess.h"

/**
 * <h1> Fetch-And-Add Array Queue </h1>
 *
 * Each node has one array but we don't search for a vacant entry. Instead, we
 * use FAA to obtain an index in the array, for enqueueing or dequeuing.
 *
 * There are some similarities between this queue and the basic queue in YMC:
 * http://chaoran.me/assets/pdf/wfq-ppopp16.pdf
 * but it's not the same because the queue in listing 1 is obstruction-free, while
 * our algorithm is lock-free.
 * In FAAArrayQueue eventually a new node will be inserted (using Michael-Scott's
 * algorithm) and it will have an item pre-filled in the first position, which means
 * that at most, after BUFFER_SIZE steps, one item will be enqueued (and it can then
 * be dequeued). This kind of progress is lock-free.
 *
 * Each entry in the array may contain one of three possible values:
 * - A valid item that has been enqueued;
 * - nullptr, which means no item has yet been enqueued in that position;
 * - taken, a special value that means there was an item but it has been dequeued;
 *
 * Enqueue algorithm: FAA + CAS(null,item)
 * Dequeue algorithm: FAA + CAS(item,taken)
 * Consistency: Linearizable
 * enqueue() progress: lock-free
 * dequeue() progress: lock-free
 * Memory Reclamation: Hazard Pointers (lock-free)
 * Uncontended enqueue: 1 FAA + 1 CAS + 1 HP
 * Uncontended dequeue: 1 FAA + 1 CAS + 1 HP
 *
 *
 * <p>
 * Lock-Free Linked List as described in Maged Michael and Michael Scott's paper:
 * {@link http://www.cs.rochester.edu/~scott/papers/1996_PODC_queues.pdf}
 * <a href="http://www.cs.rochester.edu/~scott/papers/1996_PODC_queues.pdf">
 * Simple, Fast, and Practical Non-Blocking and Blocking Concurrent Queue Algorithms</a>
 * <p>
 * The paper on Hazard Pointers is named "Hazard Pointers: Safe Memory
 * Reclamation for Lock-Free objects" and it is available here:
 * http://web.cecs.pdx.edu/~walpole/class/cs510/papers/11.pdf
 *
 * @author Pedro Ramalhete
 * @author Andreia Correia
 */
template<typename T>
class FAAArrayQueue 
{
    static constexpr long BUFFER_SIZE = 1024;  // 1024

private:
    struct Node 
    {
        std::atomic<int>   deqidx;
        std::atomic<T*>    items[BUFFER_SIZE];
        std::atomic<int>   enqidx;
        std::atomic<Node*> next;

        // Start with the first entry pre-filled and enqidx at 1
        Node(T* item) : deqidx{0}, enqidx{1}, next{nullptr} 
        {
            items[0].store(item, std::memory_order_relaxed);
            for (long i = 1; i < BUFFER_SIZE; i++) 
            {
                items[i].store(nullptr, std::memory_order_relaxed);
            }
        }

        bool casNext(Node *cmp, Node *val) 
        {
            return next.compare_exchange_strong(cmp, val);
        }
    };

    bool casTail(Node *cmp, Node *val) 
    {
		return tail.compare_exchange_strong(cmp, val);
	}

    bool casHead(Node *cmp, Node *val) 
    {
        return head.compare_exchange_strong(cmp, val);
    }

    // Pointers to head and tail of the list
    alignas(PLATFORM_CACHE_LINE_SIZE * 2) std::atomic<Node*> head;
    alignas(PLATFORM_CACHE_LINE_SIZE * 2) std::atomic<Node*> tail;

	FHazardPointerCollection Hazards;   
	inline static T* GetTakenPointer()
	{
		return reinterpret_cast<T*>(~uintptr_t(0));
	}

public:
    FAAArrayQueue()
    {
        Node* sentinelNode = new Node(nullptr);
        sentinelNode->enqidx.store(0, std::memory_order_relaxed);
        head.store(sentinelNode, std::memory_order_relaxed);
        tail.store(sentinelNode, std::memory_order_relaxed);
    }

    ~FAAArrayQueue() 
    {
        while (dequeue() != nullptr); // Drain the queue
        delete head.load();            // Delete the last node
    }

	class EnqueueHazard : private THazardPointer<Node, true>
	{
		friend class FAAArrayQueue<T>;
		inline EnqueueHazard(std::atomic<Node*>& Hazard, FHazardPointerCollection& Collection) : THazardPointer<Node, true>(Hazard, Collection)
		{}

	public:
		inline EnqueueHazard() = default;
		inline EnqueueHazard(EnqueueHazard&& Hazard) : THazardPointer<Node, true>(MoveTemp(Hazard))
		{}

		inline EnqueueHazard& operator=(EnqueueHazard&& Other)
		{
			static_cast<THazardPointer<Node, true>&>(*this) = MoveTemp(Other);
			return *this;
		}
	};

private:
	template<typename HazardType>
    void enqueueInternal(T* item, HazardType& Hazard) 
    {
        checkSlow(item); 
        while (true) 
        {
            Node* ltail = Hazard.Get();
            const int idx = ltail->enqidx.fetch_add(1);
            if (idx > BUFFER_SIZE-1) 
            { // This node is full
                if (ltail != tail.load())
                {
                    continue;
                }
                Node* lnext = ltail->next.load();
                if (lnext == nullptr) 
                {
                    Node* newNode = new Node(item);
                    if (ltail->casNext(nullptr, newNode)) 
                    {
                        casTail(ltail, newNode);
						Hazard.Retire();
                        return;
                    }
                    delete newNode;
                } 
                else 
                {
                    casTail(ltail, lnext);
                }
                continue;
            }
            T* itemnull = nullptr;
            if (ltail->items[idx].compare_exchange_strong(itemnull, item)) 
            {
				Hazard.Retire();
                return;
            }
        }
    }

public:
	inline EnqueueHazard getTailHazard()
	{
		return {tail, Hazards};
	};

	inline void enqueue(T* item, EnqueueHazard& Hazard) 
	{
		enqueueInternal(item, Hazard);
	}

	inline void enqueue(T* item)
	{
		THazardPointer<Node> Hazard(tail, Hazards);
		enqueueInternal(item, Hazard);
	}

	class DequeueHazard : private THazardPointer<Node, true>
	{
		friend class FAAArrayQueue<T>;
		inline DequeueHazard(std::atomic<Node*>& Hazard, FHazardPointerCollection& Collection) : THazardPointer<Node, true>(Hazard, Collection)
		{}

	public:
		inline DequeueHazard() = default;
		inline DequeueHazard(DequeueHazard&& Hazard) : THazardPointer<Node, true>(MoveTemp(Hazard))
		{}

		inline DequeueHazard& operator=(DequeueHazard&& Other)
		{
			static_cast<THazardPointer<Node, true>&>(*this) = MoveTemp(Other);
			return *this;
		}
	};

private:
	template<typename HazardType>
    T* dequeueInternal(HazardType& Hazard) 
    {
        while (true) 
        {
			Node* lhead = Hazard.Get();
            if (lhead->deqidx.load() >= lhead->enqidx.load() && lhead->next.load() == nullptr) 
            {
                break;
            }
            const int idx = lhead->deqidx.fetch_add(1);
            if (idx > BUFFER_SIZE-1) 
            { // This node has been drained, check if there is another one
                Node* lnext = lhead->next.load();
				if (lnext == nullptr)
				{
					break;  // No more nodes in the queue
				}
				if (casHead(lhead, lnext))
				{
					Hazard.Retire();
					Hazards.Delete(lhead);
				}
                continue;
            }

			// When there is more consumers than producers we can end up stealing
			// empty slots that producers have reserved but not yet had time to write into.
			// This leads to a lot of retries on the producers side so help this case
			// by spinning just a little bit when we know the deqidx we got is valid
			// and is likely to be written to very soon.
			if (lhead->items[idx].load() == nullptr && idx <= lhead->enqidx.load())
			{
				for (int32 Try = 0; lhead->items[idx].load() == nullptr && Try < 10; ++Try)
				{
					FPlatformProcess::Yield();
				}
			}

            T* item = lhead->items[idx].exchange(GetTakenPointer());
			if (item == nullptr)
			{
				continue;
			}
			Hazard.Retire();
            return item;
        }
		Hazard.Retire();
        return nullptr;
    }

public:
	inline T* dequeue(DequeueHazard& Hazard) 
	{
		return dequeueInternal(Hazard);
	}

	inline DequeueHazard getHeadHazard()
	{
		return {head, Hazards};
	};

	inline T* dequeue()
	{
		THazardPointer<Node> Hazard(head, Hazards);
		return dequeueInternal(Hazard);
	}
};

#endif /* _FAA_ARRAY_QUEUE_HP_H_ */
