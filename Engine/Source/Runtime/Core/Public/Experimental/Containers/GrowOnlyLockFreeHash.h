// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Misc/ScopeLock.h"

// Hash table with fast lock free reads, that only supports insertion of items, and no modification of
// values.  KeyType must be an integer.  EntryType should be a POD with an identifiable "empty" state
// that can't occur in the table, and include the following member functions:
//
//		KeyType GetKey() const;											// Get the key from EntryType
//		ValueType GetValue() const;										// Get the value from EntryType
//		bool IsEmpty() const;											// Query whether EntryType is empty
//		void SetKeyValue(KeyType Key, ValueType Value);					// Write key and value into EntryType (ATOMICALLY!  See below)
//		static uint32 KeyHash(KeyType Key);								// Convert Key to more well distributed hash
//		static void ClearEntries(EntryType* Entries, int32 EntryCount);	// Fill an array of entries with empty values
//
// The function "SetKeyValue" must be multi-thread safe when writing new items!  This means writing the
// Key last and atomically, or writing the entire EntryType in a single write (say if the key and value
// are packed into a single integer word).  Inline is recommended, since these functions are called a
// lot in the inner loop of the algorithm.  A simple implementation of "KeyHash" can just return the
// Key (if it's already reasonable as a hash), or mix the bits if better distribution is required.  A
// simple implementation of "ClearEntries" can just be a memset, if zero represents an empty entry.
//
// A set can be approximated by making "GetValue" a nop function, and just paying attention to the bool
// result from FindEntry, although you do need to either reserve a certain Key as invalid, or add
// space to store a valid flag as the Value.  This class should only be used for small value types, as
// the values are embedded into the hash table, and not stored separately.
//
// Writes are implemented using a lock -- it would be possible to make writes lock free (or lock free
// when resizing doesn't occur), but it adds complexity.  If we were to go that route, it would make
// sense to create a fully generic lock free set, which would be much more involved to implement and
// validate than this simple class, and might also offer somewhat worse read perf.  Lock free containers
// that support item removal either need additional synchronization overhead on readers, so writers can
// tell if a reader is active and spin, or need graveyard markers and a garbage collection pass called
// periodically, which makes it no longer a simple standalone container.
//
// Lock free reads are accomplished by the reader atomically pulling the hash table pointer from the
// class.  The hash table is self contained, with its size stored in the table itself, and hash tables
// are not freed until the class's destruction.  So if the table needs to be reallocated due to a write,
// active readers will still have valid memory.  This does mean that tables leak, but worst case, you
// end up with half of the memory being waste.  It would be possible to garbage collect the excess
// tables, but you'd need some kind of global synchronization to make sure no readers are active.
//
// Besides cleanup of wasted tables, it might be useful to provide a function to clear a table.  This
// would involve clearing the Key for all the elements in the table (but leaving the memory allocated),
// and can be done safely with active readers.  It's not possible to safely remove individual items due
// to the need to potentially move other items, which would break an active reader that has already
// searched past a moved item.  But in the case of removing all items, we don't care when a reader fails,
// it's expected that eventually all readers will fail, regardless of where they are searching.  A clear
// function could be useful if a lot of the data you are caching is no longer used, and you want to
// reset the cache.
//
template<typename EntryType, typename KeyType, typename ValueType>
class TGrowOnlyLockFreeHash
{
public:
	TGrowOnlyLockFreeHash(FMalloc* InMalloc)
		: Malloc(InMalloc), HashTable(nullptr)
	{}

	~TGrowOnlyLockFreeHash()
	{
		FHashHeader* HashTableNext;
		for (FHashHeader* HashTableCurrent = HashTable; HashTableCurrent; HashTableCurrent = HashTableNext)
		{
			HashTableNext = HashTableCurrent->Next;

			Malloc->Free(HashTableCurrent);
		}
	}

	/**
	 * Preallocate the hash table to a certain size
	 * @param Count - Number of EntryType elements to allocate
	 * @warning Can only be called once, and only before any items have been added!
	 */
	void Reserve(int32 Count)
	{
		FScopeLock _(&WriteCriticalSection);
		check(HashTable.load(std::memory_order_relaxed) == nullptr);

		if (Count <= 0)
		{
			Count = DEFAULT_INITIAL_SIZE;
		}
		Count = FPlatformMath::RoundUpToPowerOfTwo(Count);
		FHashHeader* HashTableLocal = (FHashHeader*)Malloc->Malloc(sizeof(FHashHeader) + (Count - 1) * sizeof(EntryType));

		HashTableLocal->Next = nullptr;
		HashTableLocal->TableSize = Count;
		HashTableLocal->Used = 0;
		EntryType::ClearEntries(HashTableLocal->Elements, Count);

		HashTable.store(HashTableLocal, std::memory_order_release);
	}

	/**
	 * Find an entry in the hash table
	 * @param Key - Key to search for
	 * @param OutValue - Memory location to write result value to.  Left unmodified if Key isn't found.
	 * @param bIsAlreadyInTable - Optional result for whether key was found in table.
	 */
	void Find(KeyType Key, ValueType *OutValue, bool* bIsAlreadyInTable = nullptr) const
	{
		FHashHeader* HashTableLocal = HashTable.load(std::memory_order_acquire);
		if (HashTableLocal)
		{
			uint32 TableMask = HashTableLocal->TableSize - 1;

			// Linear probing
			for (uint32 TableIndex = EntryType::KeyHash(Key) & TableMask; !HashTableLocal->Elements[TableIndex].IsEmpty(); TableIndex = (TableIndex + 1) & TableMask)
			{
				if (HashTableLocal->Elements[TableIndex].GetKey() == Key)
				{
					if (OutValue)
					{
						*OutValue = HashTableLocal->Elements[TableIndex].GetValue();
					}
					if (bIsAlreadyInTable)
					{
						*bIsAlreadyInTable = true;
					}
					return;
				}
			}
		}

		if (bIsAlreadyInTable)
		{
			*bIsAlreadyInTable = false;
		}
	}

	/**
	 * Add an entry with the given Key to the hash table, will do nothing if the item already exists
	 * @param Key - Key to add
	 * @param Value - Value to add for key
	 * @param bIsAlreadyInTable -- Optional result for whether item was already in table
	 */
	void Emplace(KeyType Key, ValueType Value, bool* bIsAlreadyInTable = nullptr)
	{
		FScopeLock _(&WriteCriticalSection);

		// After locking, check if the item is already in the hash table.
		ValueType ValueIgnore;
		bool bFindResult;
		Find(Key, &ValueIgnore, &bFindResult);
		if (bFindResult == true)
		{
			if (bIsAlreadyInTable)
			{
				*bIsAlreadyInTable = true;
			}
			return;
		}

		// Check if there is space in the hash table for a new item.  We resize when the hash
		// table gets half full or more.  @todo:  allow client to specify max load factor?
		FHashHeader* HashTableLocal = HashTable;

		if (!HashTableLocal || (HashTableLocal->Used >= HashTableLocal->TableSize / 2))
		{
			int32 GrowCount = HashTableLocal ? HashTableLocal->TableSize * 2 : DEFAULT_INITIAL_SIZE;
			FHashHeader* HashTableGrow = (FHashHeader*)Malloc->Malloc(sizeof(FHashHeader) + (GrowCount - 1) * sizeof(EntryType));

			HashTableGrow->Next = HashTableLocal;
			HashTableGrow->TableSize = GrowCount;
			HashTableGrow->Used = 0;
			EntryType::ClearEntries(HashTableGrow->Elements, GrowCount);

			if (HashTableLocal)
			{
				// Copy existing elements from the old table to the new table
				for (int32 TableIndex = 0; TableIndex < HashTableLocal->TableSize; TableIndex++)
				{
					EntryType& Entry = HashTableLocal->Elements[TableIndex];
					if (!Entry.IsEmpty())
					{
						HashInsertInternal(HashTableGrow, Entry.GetKey(), Entry.GetValue());
					}
				}
			}

			HashTableLocal = HashTableGrow;
			HashTable.store(HashTableGrow, std::memory_order_release);
		}

		// Then add our new item
		HashInsertInternal(HashTableLocal, Key, Value);

		if (bIsAlreadyInTable)
		{
			*bIsAlreadyInTable = false;
		}
	}

	void FindOrAdd(KeyType Key, ValueType Value, bool* bIsAlreadyInTable = nullptr)
	{
		// Attempt to find the item lock free, before calling "Emplace", which locks the container
		bool bFindResult;
		ValueType IgnoreResult;
		Find(Key, &IgnoreResult, &bFindResult);
		if (bFindResult)
		{
			if (bIsAlreadyInTable)
			{
				*bIsAlreadyInTable = true;
			}
			return;
		}

		Emplace(Key, Value, bIsAlreadyInTable);
	}

private:
	struct FHashHeader
	{
		FHashHeader* Next;			// Old buffers are stored in a linked list for cleanup
		int32 TableSize;
		int32 Used;
		EntryType Elements[1];		// Variable sized
	};

	FMalloc* Malloc;
	std::atomic<FHashHeader*> HashTable;
	FCriticalSection	WriteCriticalSection;

	static constexpr int32 DEFAULT_INITIAL_SIZE = 1024;

	static void HashInsertInternal(FHashHeader* HashTableLocal, KeyType Key, ValueType Value)
	{
		int32 TableMask = HashTableLocal->TableSize - 1;

		// Linear probing
		for (int32 TableIndex = EntryType::KeyHash(Key) & TableMask;; TableIndex = (TableIndex + 1) & TableMask)
		{
			if (HashTableLocal->Elements[TableIndex].IsEmpty())
			{
				HashTableLocal->Elements[TableIndex].SetKeyValue(Key, Value);
				HashTableLocal->Used++;
				break;
			}
		}
	}
};
