// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformMutex.h"
#include "HAL/PlatformTLS.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"

#include <atomic>

class FHazardPointerCollection;

namespace HazardPointer_Impl
{
/**
*	FHazardDeleter is used to erase the type of a class so that we can call the correct destructor
**/
class FHazardDeleter
{
	friend class ::FHazardPointerCollection;

protected:
	void* Pointer;
	FHazardDeleter(void* InPointer) : Pointer(InPointer) {}

	virtual void Delete() 
	{
		check(false);
	};

public:
	FHazardDeleter(const FHazardDeleter& Other)
	{
		//deliberately stomp over the vtable pointer so that this FHazardDeleter becomes a THazardDeleter
		FPlatformMemory::Memcpy(this, &Other, sizeof(FHazardDeleter)); //-V598
	}

	FHazardDeleter& operator= (const FHazardDeleter& Other)
	{
		//deliberately stomp over the vtable pointer so that this FHazardDeleter becomes a THazardDeleter
		FPlatformMemory::Memcpy(this, &Other, sizeof(FHazardDeleter)); //-V598
		return *this;
	}

	bool operator== (const FHazardDeleter& Other) const
	{
		return Other.Pointer == Pointer;
	}
};
}
/**
*	FHazardPointerCollection is a collection that is used to aquire a Hazardpointer
**/
class FHazardPointerCollection
{
	template<typename, bool>
	friend class THazardPointer;

	struct FTlsData
	{
		TArray<HazardPointer_Impl::FHazardDeleter> ReclamationList;
		double TimeOfLastCollection = .0;

		~FTlsData()
		{
			for (HazardPointer_Impl::FHazardDeleter& Deleter : ReclamationList)
			{
				Deleter.Delete();
			}
			ReclamationList.Empty();
		}
	};

	class alignas(PLATFORM_CACHE_LINE_SIZE * 2) FHazardRecord
	{
		friend class FHazardPointerCollection;

		template<typename, bool>
		friend class THazardPointer;
		static constexpr uintptr_t FreeHazardEntry = ~uintptr_t(0);

		std::atomic<uintptr_t> Hazard{ FreeHazardEntry };
#ifdef _MSC_VER
	// UE_DEPRECATED - this is a workaround for https://developercommunity.visualstudio.com/t/VS-2022-1790-Preview-10-__builtin_arr/10519788 and should be removed after
	//                 after 17.9 is no longer supported.
	public:
		FHazardRecord() = default;
	private:
#else
		FHazardRecord() = default;
#endif

		inline void* GetHazard() const
		{
			return reinterpret_cast<void*>(Hazard.load(std::memory_order_acquire));
		}

		//assign hazard pointer once acquired
		[[nodiscard]] inline void* SetHazard(void* InHazard)
		{
			Hazard.store(reinterpret_cast<uintptr_t>(InHazard), std::memory_order_release);
			std::atomic_thread_fence(std::memory_order_seq_cst);
			return reinterpret_cast<void*>(Hazard.load(std::memory_order_acquire));
		}

		//this thread wants to re-use the slot but does not want to hold onto the pointer
		inline void Retire()
		{
			Hazard.store(0, std::memory_order_release);
		}

		//good to reuse by another thread
		inline void Release()
		{
			Hazard.store(FreeHazardEntry, std::memory_order_release);
		}
	};

	template<typename D>
	class THazardDeleter final : public HazardPointer_Impl::FHazardDeleter
	{
	public:
		THazardDeleter(D* InPointer) : HazardPointer_Impl::FHazardDeleter(reinterpret_cast<void*>(InPointer))
		{
			static_assert(sizeof(THazardDeleter<D>) == sizeof(FHazardDeleter), "Size mismatch: we want to store and THazardDeleter in an FHazardDeleter array");
		}

		void Delete() override
		{
			D* Ptr = reinterpret_cast<D*>(Pointer);
			delete Ptr;
		}
	};

	static constexpr uint32 HazardChunkSize = 32;
	struct FHazardRecordChunk
	{
		FHazardRecord Records[HazardChunkSize] = {};
		std::atomic<FHazardRecordChunk*> Next{ nullptr };

		inline void* operator new(size_t Size)
		{
			return FMemory::Malloc(Size, 128u);
		}

		inline void operator delete(void* Ptr)
		{
			FMemory::Free(Ptr);
		}
	};

	FHazardRecordChunk Head;

	UE::FPlatformRecursiveMutex AllTlsVariablesCS;
	UE::FPlatformRecursiveMutex HazardRecordBlocksCS;
	TArray<FTlsData*> AllTlsVariables;
	TArray<FHazardRecordChunk*> HazardRecordBlocks;

	uint32 CollectablesTlsSlot = FPlatformTLS::InvalidTlsSlot;
	std::atomic_uint TotalNumHazardRecords{ HazardChunkSize };

	void Collect(TArray<HazardPointer_Impl::FHazardDeleter>& Collectables);

	//mark pointer for deletion
	CORE_API void Delete(const HazardPointer_Impl::FHazardDeleter& Deleter, int32 CollectLimit);

	template<bool Cached>
	CORE_API FHazardRecord* Grow();

public:
	FHazardPointerCollection()
	{
		CollectablesTlsSlot = FPlatformTLS::AllocTlsSlot();
	}

	CORE_API ~FHazardPointerCollection();

	//grab a hazard pointer and once hazard is set the other threads leave it alone
	template<bool Cached>
	inline FHazardRecord* Acquire()
	{
		struct FPseudo
		{
			static inline uint32 GetThreadId()
			{
				static std::atomic_uint counter{0};
				uint32 value = counter.fetch_add(1, std::memory_order_relaxed);
				value = ((value >> 16) ^ value) * 0x45d9f3b;
				value = ((value >> 16) ^ value) * 0x45d9f3b;
				value = (value >> 16) ^ value;
				return value;
			}
		};

		static thread_local uint32 StartIndex = FPseudo::GetThreadId();
	
		FHazardRecordChunk* p = &Head;
		if (Cached)
		{
			// acquire pairs with the release store in Grow() that publishes a new chunk;
			// without it, the chunk constructor's writes to Records[] are not
			// synchronized with the CAS below.
			p = p->Next.load(std::memory_order_acquire);
			goto TestCondition;
		}

		//search HazardPointerList for an empty Entry
		do
		{
			for (uint64 idx = 0; idx < HazardChunkSize; idx++)
			{
				uintptr_t Nullptr = 0;
				uintptr_t FreeEntry = FHazardRecord::FreeHazardEntry;
				uint64 i = (StartIndex + idx) % HazardChunkSize;
				if (p->Records[i].Hazard.compare_exchange_weak(FreeEntry, Nullptr, std::memory_order_relaxed))
				{
					checkSlow(p->Records[i].GetHazard() == nullptr);
					return &p->Records[i];
				}
			}
			// acquire pairs with the release store in Grow() that publishes a new chunk;
			// without it, the chunk constructor's writes to Records[] are not
			// synchronized with the CAS below.
			p = p->Next.load(std::memory_order_acquire);
			TestCondition:;
		} while (p);

		return Grow<Cached>();
	}

	//if we own the pointer
	template<typename D>
	inline void Delete(D* Pointer, int32 CollectLimit = -1)
	{
		if (Pointer)
		{
			Delete(THazardDeleter<D>(Pointer), CollectLimit);
		}
	}
};


/**
*	THazardPointer is used to keep an allocation alive until all threads that referenced it finished their access 
**/
template<typename H, bool Cached = false>
class THazardPointer
{
	THazardPointer(const THazardPointer&) = delete;
	THazardPointer& operator=(const THazardPointer&) = delete;
	
	std::atomic<H*>* Hazard = nullptr;
	FHazardPointerCollection::FHazardRecord* Record = nullptr;

public:
	THazardPointer(THazardPointer&& Other) : Hazard(Other.Hazard), Record(Other.Record)
	{
		if (Record)
		{
			Record->Release();
		}
		Other.Hazard = nullptr;
		Other.Record = nullptr;
	}

	THazardPointer& operator=(THazardPointer&& Other)
	{
		if (Record)
		{
			Record->Release();
		}
		Hazard = Other.Hazard;
		Record = Other.Record;
		Other.Hazard = nullptr;
		Other.Record = nullptr;
		return *this;
	}

public:
	THazardPointer() = default;

	inline THazardPointer(std::atomic<H*>& InHazard, FHazardPointerCollection& Collection)
		: Hazard(&InHazard), Record(Collection.Acquire<Cached>())
	{
		checkSlow(Record->GetHazard() == nullptr);
	}

	inline ~THazardPointer()
	{
		if (Record)
		{
			Record->Release();
		}
	}

	//retireing can be used to release the hazardpointer without reaquiring a new Hazardslot in the Collection
	inline void Retire()
	{
		checkSlow(Record);
		Record->Retire();
	}

	//use with care, because the Hazardpointer will not protect anymore and needs to be recreated
	inline void Destroy()
	{
		checkSlow(Record);
		Record->Release();
		Record = nullptr;
		Hazard = nullptr;
	}

	inline H* Get() const
	{
		checkSlow(Record);
		H* HazardPointer;
		do
		{
			HazardPointer = (H*)Record->SetHazard(Hazard->load(std::memory_order_acquire));
		} while (HazardPointer != Hazard->load(std::memory_order_acquire));
		return HazardPointer;
	}

	inline bool IsValid()
	{
		return Hazard != nullptr && Record != nullptr;
	}
};

template<typename H>
THazardPointer<H, false> MakeHazardPointer(std::atomic<H*>& InHazard, FHazardPointerCollection& Collection)
{
	return {InHazard, Collection};
}
