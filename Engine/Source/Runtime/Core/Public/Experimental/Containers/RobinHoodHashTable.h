// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Implementation of Robin Hood hash table.
#include "CoreMinimal.h"
#include "Containers/Map.h"
#include "Templates/UnrealTemplate.h"

#define RUN_HASHTABLE_CONCURENCY_CHECKS UE_BUILD_DEBUG

#if RUN_HASHTABLE_CONCURENCY_CHECKS
#define CHECK_CONCURRENT_ACCESS(x) check(x)
#else
#define CHECK_CONCURRENT_ACCESS(x)
#endif

namespace Experimental
{

namespace RobinHoodHashTable_Private
{
	template<typename, typename, typename, typename>
	class TRobinHoodHashTable;
}

class FHashType
{
public:
	explicit FHashType() : Hash(InvalidHash) {}

	inline bool operator == (FHashType Other) const
	{
		return Hash == Other.Hash;
	}

	inline bool operator != (FHashType Other) const
	{
		return Hash != Other.Hash;
	}

	using IntType = uint32;

	inline IntType AsUInt() const
	{
		return Hash;
	}

private:
	template<typename, typename, typename, typename>
	friend class RobinHoodHashTable_Private::TRobinHoodHashTable;

	inline explicit FHashType(IntType InHash) : Hash(InHash)
	{
		checkSlow(!(InHash & InvalidHash));
	}

	inline bool IsOccupied() const
	{
		return Hash != InvalidHash;
	}

	inline bool IsFree() const
	{
		return Hash == InvalidHash;
	}

	static constexpr const IntType InvalidHash = (IntType(1)) << (sizeof(IntType) * 8 - 1);
	IntType Hash;
};

class FHashElementId
{
public:
	FHashElementId() : Index(INDEX_NONE) {}
	inline FHashElementId(int32 InIndex) : Index(InIndex) {}

	inline int32 GetIndex() const
	{
		return Index;
	}

	inline bool IsValid() const
	{
		return Index != INDEX_NONE;
	}

	inline bool operator==(FHashElementId InHashElementId) const
	{
		return Index == InHashElementId.Index;
	}

private:
	int32 Index;
};

namespace RobinHoodHashTable_Private
{
	template<typename AllocatorType>
	class TFreeList
	{
		using IndexType = int32;

		struct FSpan
		{
			IndexType Start;
			IndexType End;
		};

		TArray<FSpan, AllocatorType> FreeList;
		int32 NumElements = 0;

	public:
		TFreeList()
		{
			Empty();
		}

		void Push(IndexType NodeIndex)
		{
			++NumElements;

			//find the index that points to our right side node
			int Index = 1; //exclude the dummy
			int Size = FreeList.Num() - 1;

			//start with binary search for larger lists
			while (Size > 32)
			{
				const int LeftoverSize = Size % 2;
				Size = Size / 2;

				const int CheckIndex = Index + Size;
				const int IndexIfLess = CheckIndex + LeftoverSize;

				Index = FreeList[CheckIndex].Start > NodeIndex ? IndexIfLess : Index;
			}

			//small size array optimization
			int ArrayEnd = FMath::Min(Index + Size + 1, FreeList.Num());
			while (Index < ArrayEnd)
			{
				if (FreeList[Index].Start < NodeIndex)
				{
					break;
				}
				Index++;
			}

			//can we merge with the right node
			if (Index < FreeList.Num() && FreeList[Index].End + 1 == NodeIndex)
			{
				FreeList[Index].End = NodeIndex;

				//are we filling the gap between the left and right node
				if (FreeList[Index - 1].Start - 1 == NodeIndex)
				{
					FreeList[Index - 1].Start = FreeList[Index].Start;
					FreeList.RemoveAt(Index);
				}
				return;
			}

			//can we merge with the left node
			if (FreeList[Index - 1].Start - 1 == NodeIndex)
			{
				FreeList[Index - 1].Start = NodeIndex;
				return;
			}

			//we are node that could not be merged
			FreeList.Insert(FSpan{ NodeIndex , NodeIndex }, Index);
		}

		IndexType Pop()
		{
			--NumElements;

			FSpan& Span = FreeList.Last();
			IndexType Index = Span.Start;
			checkSlow(Index != INDEX_NONE);
			if (Span.Start == Span.End)
			{
				FreeList.Pop();
				return Index;
			}
			else
			{
				Span.Start++;
				return Index;
			}
		}

		bool Contains(IndexType Index) const
		{
			for (int i = FreeList.Num() - 1; i > -0; i--)
			{
				const FSpan& Span = FreeList[i];
				if (Index >= Span.Start && Index <= Span.End)
				{
					return true;
				}
			}
			return false;
		}

		void Empty()
		{
			FreeList.Reset(1);
			//push a dummy
			FreeList.Push(FSpan{ INDEX_NONE, INDEX_NONE });
			NumElements = 0;
		}

		int Num() const
		{
			return NumElements;
		}

		SIZE_T GetAllocatedSize() const
		{
			return FreeList.GetAllocatedSize();
		}
	};

	struct FUnitType
	{
	};

	template<typename KeyType, typename ValueType>
	class TKeyValue
	{
		using FindValueType = ValueType*;
		using FindValueTypeConst = const ValueType *;
		using ElementType = TPair<const KeyType, ValueType>;

		template<typename, typename, typename, typename>
		friend class TRobinHoodHashTable;

		template<typename DeducedKeyType, typename DeducedValueType>
		inline TKeyValue(DeducedKeyType&& InKey, DeducedValueType&& InVal)
		: Pair(Forward<DeducedKeyType>(InKey)
		, Forward<DeducedValueType>(InVal))
		{
		}

		ElementType Pair;

		inline FindValueType FindImpl()
		{
			return &Pair.Value;
		}

		inline FindValueTypeConst FindImpl() const
		{
			return &Pair.Value;
		}

		template<typename DeducedValueType>
		inline void Update(DeducedValueType&& InValue)
		{
			Pair.Value = Forward<DeducedValueType>(InValue);
		}

		inline ElementType& GetElement()
		{
			return Pair;
		}

		inline const ElementType& GetElement() const
		{
			return Pair;
		}

		inline const KeyType& GetKey() const
		{
			return Pair.Key;
		}
	};

	template<typename KeyType>
	class TKeyValue<KeyType, FUnitType>
	{
		using FindValueType = const KeyType*;
		using FindValueTypeConst = FindValueType;
		using ElementType = const KeyType;

		template<typename, typename, typename, typename>
		friend class TRobinHoodHashTable;

		template<typename DeducedKeyType>
		inline TKeyValue(DeducedKeyType&& InKey, FUnitType&&)
		: Key(Forward<DeducedKeyType>(InKey))
		{
		}

		ElementType Key;

		inline FindValueType FindImpl() const
		{
			return &Key;
		}

		inline ElementType& GetElement() const
		{
			return Key;
		}

		inline const KeyType& GetKey() const
		{
			return Key;
		}
	};

	template<typename KeyType, typename ValueType, typename Hasher, typename HashMapAllocator>
	class TRobinHoodHashTable
	{
	protected:
		using InlineOneAllocatorType = TInlineAllocator<1, HashMapAllocator>;
		using KeyValueType = RobinHoodHashTable_Private::TKeyValue<KeyType, ValueType>;
		using FindValueType = typename KeyValueType::FindValueType;
		using FindValueTypeConst = typename KeyValueType::FindValueTypeConst;
		using ElementType = typename KeyValueType::ElementType;
		using IndexType = uint32;
		using SizeType = SIZE_T;

		static constexpr const IndexType LoadFactorDivisor = 3;
		static constexpr const IndexType LoadFactorQuotient = 5;
		static constexpr const IndexType InvalidIndex = ~IndexType(0);

		struct FData
		{
			FData() = default;
			FData(const FData&) = default;
			FData& operator=(const FData&) = default;
			~FData()
			{
				Empty();
			}

			SizeType GetAllocatedSize() const
			{
				return KeyVals.GetAllocatedSize() + FreeList.GetAllocatedSize();
			}

			template<typename DeducedKeyType, typename DeducedValueType>
			inline IndexType Allocate(DeducedKeyType&& Key, DeducedValueType&& Val, FHashType Hash)
			{
				CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentWriters) == 1);
				CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentReaders) == 1);
				IndexType Index;
				if (FreeList.Num() > 0)
				{
					Index = FreeList.Pop();
					::new ((void*)&KeyVals[Index]) KeyValueType{ Forward<DeducedKeyType>(Key), Forward<DeducedValueType>(Val) };
					Hashes[Index] = Hash;

				}
				else
				{
					Index = KeyVals.Num();
					KeyVals.Push(KeyValueType{ Forward<DeducedKeyType>(Key), Forward<DeducedValueType>(Val) });
					Hashes.Push(Hash);
				}
				checkSlow(Index != InvalidIndex);
				CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentReaders) == 0);
				CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentWriters) == 0);
				return Index;
			}

			inline void Deallocate(IndexType Index)
			{
				CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentWriters) == 1);
				CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentReaders) == 1);
				FreeList.Push(Index);
				Hashes[Index] = FHashType();
				KeyVals[Index].~KeyValueType();
#if RUN_HASHTABLE_CONCURENCY_CHECKS
				memset(&KeyVals[Index], 0, sizeof(KeyValueType));
#endif
				CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentReaders) == 0);
				CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentWriters) == 0);
				//TODO shrink KeyValue Array.
				//As the FreeList is sorted backwards this would work as follows:
				// - go though the FreeList front to back as long as the FreeList entry is the last
				// - than pop-off the last KeyValue WITHOUT calling its destructor again
				// - remove the entry from the FreeList (probably remove the entire span at the end of iteration, as its tail would need to be moved)
			}

			inline bool Contains(IndexType Index) const
			{
				return Index < (IndexType)KeyVals.Num() && !FreeList.Contains(Index);
			}

			inline const KeyValueType& Get(IndexType Index) const
			{
				CHECK_CONCURRENT_ACCESS(ConcurrentWriters == 0);
				CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentReaders) >= 1);
				CHECK_CONCURRENT_ACCESS(Contains(Index));
				CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentReaders) >= 0);
				CHECK_CONCURRENT_ACCESS(ConcurrentWriters == 0);
				return KeyVals[Index];
			}

			inline KeyValueType& Get(IndexType Index)
			{
				CHECK_CONCURRENT_ACCESS(ConcurrentWriters == 0);
				CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentReaders) >= 1);
				CHECK_CONCURRENT_ACCESS(Contains(Index));
				CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentReaders) >= 0);
				CHECK_CONCURRENT_ACCESS(ConcurrentWriters == 0);
				return KeyVals[Index];
			}

			inline SizeType Num() const
			{
				return SizeType(KeyVals.Num()) - SizeType(FreeList.Num());
			}

			inline IndexType GetMaxIndex() const
			{
				return KeyVals.Num();
			}

			struct FIteratorState
			{
				IndexType Index;

				inline bool operator ==(const FIteratorState& Rhs) const
				{
					return Index == Rhs.Index;
				}

				inline bool operator !=(const FIteratorState& Rhs) const
				{
					return Index != Rhs.Index;
				}
			};

			inline FIteratorState Next(FIteratorState State) const
			{
				for (;;)
				{
					State.Index = (State.Index + 1) & InvalidIndex;
					if (State.Index >= uint32(KeyVals.Num()) || Hashes[State.Index].IsOccupied())
					{
						return FIteratorState{ State.Index };
					}
				}
			}

			inline FIteratorState Start() const
			{
				return Next(FIteratorState{ InvalidIndex });
			}

			inline FIteratorState End() const
			{
				return FIteratorState{ IndexType(KeyVals.Num()) };
			}

			void Empty()
			{
				FIteratorState Iter = Start();
				FIteratorState EndIter = End();
				while (Iter != EndIter)
				{
					KeyVals[Iter.Index].~KeyValueType();
					Iter = Next(Iter);
				}
				KeyVals.SetNumUnsafeInternal(0);
				KeyVals.Empty();
				FreeList.Empty();
				Hashes.Empty();
			}

			void Reserve(SizeType ReserveNum)
			{
				KeyVals.Reserve(ReserveNum);
			}

#if RUN_HASHTABLE_CONCURENCY_CHECKS
			mutable int ConcurrentReaders = 0;
			mutable int ConcurrentWriters = 0;
#endif
			TArray<KeyValueType, HashMapAllocator> KeyVals;
			TArray<FHashType, HashMapAllocator> Hashes;
			TFreeList<InlineOneAllocatorType> FreeList;
		};

		inline IndexType ModTableSize(IndexType HashValue) const
		{
			return HashValue & SizePow2Minus1;
		}

		inline void InsertIntoTable(IndexType Index, FHashType Hash)
		{
			IndexType InsertIndex = Index;
			FHashType InsertHash = Hash;
			IndexType CurrentBucket = ModTableSize(Hash.AsUInt());
			IndexType InsertDistance = 0;
			for (;;)
			{
				IndexType OtherDistance = ModTableSize(CurrentBucket - HashData[CurrentBucket].AsUInt());

				checkSlow(HashData[CurrentBucket].IsFree() || OtherDistance <= MaximumDistance);
				checkSlow(CurrentBucket == (ModTableSize(ModTableSize(HashData[CurrentBucket].AsUInt()) + OtherDistance)));

				if (HashData[CurrentBucket].IsFree())
				{
					if (InsertDistance > MaximumDistance)
					{
						MaximumDistance = InsertDistance;
					}

					IndexData[CurrentBucket] = InsertIndex;
					HashData[CurrentBucket] = InsertHash;
					break;
				}
				else if (OtherDistance < InsertDistance)
				{
					if (InsertDistance > MaximumDistance)
					{
						MaximumDistance = InsertDistance;
					}

					IndexType OtherIndex = IndexData[CurrentBucket];
					FHashType OtherHash = HashData[CurrentBucket];
					IndexData[CurrentBucket] = InsertIndex;
					HashData[CurrentBucket] = InsertHash;

					InsertDistance = OtherDistance;
					InsertIndex = OtherIndex;
					InsertHash = OtherHash;
				}
				InsertDistance++;
				CurrentBucket = ModTableSize(CurrentBucket + 1);
			}
		}

	private:
		template<bool UpdateValue, typename DeducedKeyType, typename DeducedValueType>
		inline FHashElementId FindOrUpdateIdByHashInternal(FHashType HashValue, DeducedKeyType&& Key, DeducedValueType&& Val, bool& bIsAlreadyInMap)
		{
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentWriters) == 1);
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentReaders) == 1);

			checkSlow(HashValue == ComputeHash(Key));
			IndexType BucketIndex = ModTableSize(HashValue.AsUInt());
			const IndexType EndBucketIndex = ModTableSize(HashValue.AsUInt() + MaximumDistance + 1);
			do
			{
				if (HashValue == HashData[BucketIndex])
				{
					if (Hasher::Matches(Key, KeyValueData.Get(IndexData[BucketIndex]).GetKey()))
					{
						CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentReaders) == 0);
						CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentWriters) == 0);
						bIsAlreadyInMap = true;
						if constexpr (UpdateValue)
						{
							KeyValueData.Get(IndexData[BucketIndex]).Update(Forward<DeducedValueType>(Val));
						}
						return IndexData[BucketIndex];
					}
				}

				BucketIndex = ModTableSize(BucketIndex + 1);
			} while (BucketIndex != EndBucketIndex);

			if ((KeyValueData.Num() * LoadFactorQuotient) >= (SizePow2Minus1 * LoadFactorDivisor))
			{
				TArray<IndexType, InlineOneAllocatorType> IndexDataOld = MoveTemp(IndexData);
				TArray<FHashType, InlineOneAllocatorType> HashDataOld = MoveTemp(HashData);
				IndexType OldSizePow2Minus1 = SizePow2Minus1;
				SizePow2Minus1 = SizePow2Minus1 * 2 + 1;
				MaximumDistance = 0;
				IndexData.Reserve(SizePow2Minus1 + 1); 
				IndexData.AddUninitialized(SizePow2Minus1 + 1);
				HashData.Reserve(SizePow2Minus1 + 1);  
				HashData.AddDefaulted(SizePow2Minus1 + 1);

				for (IndexType Index = 0; Index <= OldSizePow2Minus1; Index++)
				{
					if (HashDataOld[Index].IsOccupied())
					{
						InsertIntoTable(IndexDataOld[Index], HashDataOld[Index]);
					}
				}
			}

			IndexType InsertIndex = KeyValueData.Allocate(Forward<DeducedKeyType>(Key), Forward<DeducedValueType>(Val), HashValue);
			InsertIntoTable(InsertIndex, HashValue);

			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentReaders) == 0);
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentWriters) == 0);

			bIsAlreadyInMap = false;
			return FHashElementId(InsertIndex);
		}

	protected:
		template<typename DeducedKeyType, typename DeducedValueType>
		inline FHashElementId FindOrAddIdByHash(FHashType HashValue, DeducedKeyType&& Key, DeducedValueType&& Val, bool& bIsAlreadyInMap)
		{
			return FindOrUpdateIdByHashInternal<false>(HashValue, Forward<DeducedKeyType>(Key), Forward<DeducedValueType>(Val), bIsAlreadyInMap);
		}

		template<typename DeducedKeyType, typename DeducedValueType>
		inline FHashElementId FindOrAddId(DeducedKeyType&& Key, DeducedValueType&& Val, bool& bIsAlreadyInMap)
		{
			FHashType HashValue = ComputeHash(Key);
			return FindOrAddIdByHash(HashValue, Forward<DeducedKeyType>(Key), Forward<DeducedValueType>(Val), bIsAlreadyInMap);
		}

		template<typename DeducedKeyType, typename DeducedValueType>
		inline FindValueType FindOrAdd(DeducedKeyType&& Key, DeducedValueType&& Val, bool& bIsAlreadyInMap)
		{
			FHashElementId Id = FindOrAddId(Forward<DeducedKeyType>(Key), Forward<DeducedValueType>(Val), bIsAlreadyInMap);
			return KeyValueData.Get(Id.GetIndex()).FindImpl();
		}

		template<typename DeducedKeyType, typename DeducedValueType>
		inline FHashElementId UpdateIdByHash(FHashType HashValue, DeducedKeyType&& Key, DeducedValueType&& Val, bool& bIsAlreadyInMap)
		{
			return FindOrUpdateIdByHashInternal<true>(HashValue, Forward<DeducedKeyType>(Key), Forward<DeducedValueType>(Val), bIsAlreadyInMap);
		}

		template<typename DeducedKeyType, typename DeducedValueType>
		inline FHashElementId UpdateId(DeducedKeyType&& Key, DeducedValueType&& Val, bool& bIsAlreadyInMap)
		{
			FHashType HashValue = ComputeHash(Key);
			return UpdateIdByHash(HashValue, Forward<DeducedKeyType>(Key), Forward<DeducedValueType>(Val), bIsAlreadyInMap);
		}

		template<typename DeducedKeyType, typename DeducedValueType>
		inline FindValueType Update(DeducedKeyType&& Key, DeducedValueType&& Val, bool& bIsAlreadyInMap)
		{
			FHashElementId Id = UpdateId(Forward<DeducedKeyType>(Key), Forward<DeducedValueType>(Val), bIsAlreadyInMap);
			return KeyValueData.Get(Id.GetIndex()).FindImpl();
		}

		TRobinHoodHashTable()
		{
			IndexData.Reserve(1); IndexData.AddUninitialized();
			HashData.Reserve(1);  HashData.AddDefaulted();
		}

		TRobinHoodHashTable(const TRobinHoodHashTable& Other)
		{
			SizePow2Minus1 = Other.SizePow2Minus1;
			MaximumDistance = Other.MaximumDistance;
			KeyValueData = Other.KeyValueData;

			IndexData.Reserve(SizePow2Minus1 + 1); 
			IndexData.AddUninitialized(SizePow2Minus1 + 1);
			HashData.Reserve(SizePow2Minus1 + 1);  
			HashData.AddUninitialized(SizePow2Minus1 + 1);

			for (uint32 Idx = 0; Idx <= SizePow2Minus1; Idx++)
			{
				IndexData[Idx] = Other.IndexData[Idx];
				HashData[Idx] = Other.HashData[Idx];
			}
		}

		TRobinHoodHashTable& operator=(const TRobinHoodHashTable& Other)
		{
			if (this != &Other)
			{
				SizePow2Minus1 = Other.SizePow2Minus1;
				MaximumDistance = Other.MaximumDistance;
				KeyValueData = Other.KeyValueData;

				IndexData.Reserve(SizePow2Minus1 + 1); 
				IndexData.AddUninitialized(SizePow2Minus1 + 1);
				HashData.Reserve(SizePow2Minus1 + 1);  
				HashData.AddUninitialized(SizePow2Minus1 + 1);

				for (uint32 Idx = 0; Idx <= SizePow2Minus1; Idx++)
				{
					IndexData[Idx] = Other.IndexData[Idx];
					HashData[Idx] = Other.HashData[Idx];
				}
			}

			return *this;
		}

		TRobinHoodHashTable(TRobinHoodHashTable&& Other)
		{
			SizePow2Minus1 = Other.SizePow2Minus1;
			MaximumDistance = Other.MaximumDistance;
			KeyValueData = MoveTemp(Other.KeyValueData);

			IndexData = MoveTemp(Other.IndexData);
			HashData = MoveTemp(Other.HashData);

			Other.Empty();
		}

		TRobinHoodHashTable& operator=(TRobinHoodHashTable&& Other)
		{
			if (this != &Other)
			{
				SizePow2Minus1 = Other.SizePow2Minus1;
				MaximumDistance = Other.MaximumDistance;
				KeyValueData = MoveTemp(Other.KeyValueData);

				IndexData = MoveTemp(Other.IndexData);
				HashData = MoveTemp(Other.HashData);

				Other.Empty();
			}

			return *this;
		}

	public:
		inline static FHashType ComputeHash(const KeyType& Key)
		{
			typename FHashType::IntType HashValue = Hasher::GetKeyHash(Key);
			constexpr typename FHashType::IntType HashBits = (~(1 << (sizeof(typename FHashType::IntType) * 8 - 1)));
			return FHashType(HashValue & HashBits);
		}

		class FIteratorType
		{
			typename FData::FIteratorState State;
			FData& Data;

			template<typename, typename, typename, typename>
			friend class TRobinHoodHashTable;

			inline FIteratorType(FData& InData, bool bIsStartIterator) : Data(InData)
			{
				if (bIsStartIterator)
				{
					State = Data.Start();
				}
				else
				{
					State = Data.End();
				}
			}

		public:
			inline bool operator ==(const FIteratorType& Rhs) const
			{
				return State == Rhs.State && &Data == &Rhs.Data;
			}

			inline bool operator !=(const FIteratorType& Rhs) const
			{
				return State != Rhs.State || &Data != &Rhs.Data;
			}

			inline ElementType& operator*() const
			{
				return Data.Get(State.Index).GetElement();
			}

			inline FIteratorType& operator++()
			{
				State = Data.Next(State);
				return *this;
			}

			inline FHashElementId GetElementId() const
			{
				return FHashElementId(State.Index);
			}
		};

		inline FIteratorType begin()
		{
			return FIteratorType(KeyValueData, true);
		}

		inline FIteratorType end()
		{
			return FIteratorType(KeyValueData, false);
		}

		class FConstIteratorType
		{
			typename FData::FIteratorState State;
			const FData& Data;

			template<typename, typename, typename, typename>
			friend class TRobinHoodHashTable;

			inline FConstIteratorType(const FData& InData, bool bIsStartIterator) : Data(InData)
			{
				if (bIsStartIterator)
				{
					State = Data.Start();
				}
				else
				{
					State = Data.End();
				}
			}

		public:
			inline bool operator ==(const FConstIteratorType& Rhs) const
			{
				return State == Rhs.State && &Data == &Rhs.Data;
			}

			inline bool operator !=(const FConstIteratorType& Rhs) const
			{
				return State != Rhs.State || &Data != &Rhs.Data;
			}

			inline const ElementType& operator*() const
			{
				return Data.Get(State.Index).GetElement();
			}

			inline FConstIteratorType& operator++()
			{
				State = Data.Next(State);
				return *this;
			}

			inline FHashElementId GetElementId() const
			{
				return FHashElementId(State.Index);
			}
		};

		inline FConstIteratorType begin() const
		{
			return FConstIteratorType(KeyValueData, true);
		}

		inline FConstIteratorType end() const
		{
			return FConstIteratorType(KeyValueData, false);
		}

		SizeType GetAllocatedSize() const
		{
			return KeyValueData.GetAllocatedSize() + IndexData.GetAllocatedSize() + HashData.GetAllocatedSize();
		}

		inline int32 Num() const
		{
			return (int32)KeyValueData.Num();
		}

		inline IndexType GetMaxIndex() const
		{
			return KeyValueData.GetMaxIndex();
		}

		inline ElementType& GetByElementId(FHashElementId Id)
		{
			return KeyValueData.Get(Id.GetIndex()).GetElement();
		}

		inline const ElementType& GetByElementId(FHashElementId Id) const
		{
			return KeyValueData.Get(Id.GetIndex()).GetElement();
		}

		inline bool ContainsElementId(FHashElementId Id) const
		{
			return KeyValueData.Contains(Id.GetIndex());
		}

		inline FHashElementId FindIdByHash(const FHashType HashValue, const KeyType& ComparableKey) const
		{
			CHECK_CONCURRENT_ACCESS(ConcurrentWriters == 0);
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentReaders) >= 1);

			checkSlow(HashValue == ComputeHash(ComparableKey));
			IndexType BucketIndex = ModTableSize(HashValue.AsUInt());
			const IndexType EndBucketIndex = ModTableSize(HashValue.AsUInt() + MaximumDistance + 1);
			do
			{
				if (HashValue == HashData[BucketIndex])
				{
					if (Hasher::Matches(ComparableKey, KeyValueData.Get(IndexData[BucketIndex]).GetKey()))
					{
						CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentReaders) >= 0);
						CHECK_CONCURRENT_ACCESS(ConcurrentWriters == 0);
						return FHashElementId(IndexData[BucketIndex]);
					}
				}

				BucketIndex = ModTableSize(BucketIndex + 1);
			} while (BucketIndex != EndBucketIndex);

			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentReaders) >= 0);
			CHECK_CONCURRENT_ACCESS(ConcurrentWriters == 0);

			return FHashElementId();
		}

		inline FHashElementId FindId(const KeyType& Key) const
		{
			const FHashType HashValue = ComputeHash(Key);
			return FindIdByHash(HashValue, Key);
		}

		FindValueType FindByHash(const FHashType HashValue, const KeyType& Key)
		{
			FHashElementId Id = FindIdByHash(HashValue, Key);
			if (Id.IsValid())
			{
				return KeyValueData.Get(Id.GetIndex()).FindImpl();
			}

			return nullptr;
		}

		FindValueType Find(const KeyType& Key)
		{
			FHashElementId Id = FindId(Key);
			if (Id.IsValid())
			{
				return KeyValueData.Get(Id.GetIndex()).FindImpl();
			}

			return nullptr;
		}

		const FindValueType FindByHash(const FHashType HashValue, const KeyType& Key) const
		{
			FHashElementId Id = FindIdByHash(HashValue, Key);
			if (Id.IsValid())
			{
				return KeyValueData.Get(Id.GetIndex()).FindImpl();
			}

			return nullptr;
		}

		FindValueTypeConst Find(const KeyType& Key) const
		{
			FHashElementId Id = FindId(Key);
			if (Id.IsValid())
			{
				return KeyValueData.Get(Id.GetIndex()).FindImpl();
			}

			return nullptr;
		}

		bool RemoveByHash(const FHashType HashValue, const KeyType& ComparableKey)
		{
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentWriters) == 1);
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentReaders) == 1);

			bool bIsFoundInMap = false;
			checkSlow(HashValue == ComputeHash(ComparableKey))
			IndexType BucketIndex = ModTableSize(HashValue.AsUInt());
			const IndexType EndBucketIndex = ModTableSize(HashValue.AsUInt() + MaximumDistance + 1);
			do
			{
				if (HashValue == HashData[BucketIndex])
				{
					if (Hasher::Matches(ComparableKey, KeyValueData.Get(IndexData[BucketIndex]).GetKey()))
					{
						KeyValueData.Deallocate(IndexData[BucketIndex]);
						HashData[BucketIndex] = FHashType();
						bIsFoundInMap = true;
						break;
					}
				}

				BucketIndex = ModTableSize(BucketIndex + 1);
			} while (BucketIndex != EndBucketIndex);

			if (bIsFoundInMap && (KeyValueData.Num() * LoadFactorQuotient * 4) < (SizePow2Minus1 * LoadFactorDivisor))
			{
				TArray<IndexType, InlineOneAllocatorType> IndexDataOld = MoveTemp(IndexData);
				TArray<FHashType, InlineOneAllocatorType> HashDataOld = MoveTemp(HashData);
				IndexType OldSizePow2Minus1 = SizePow2Minus1;
				SizePow2Minus1 = SizePow2Minus1 / 2;
				MaximumDistance = 0;
				IndexData.Reserve(SizePow2Minus1 + 1); 
				IndexData.AddUninitialized(SizePow2Minus1 + 1);
				HashData.Reserve(SizePow2Minus1 + 1);  
				HashData.AddDefaulted(SizePow2Minus1 + 1);

				for (IndexType Index = 0; Index <= OldSizePow2Minus1; Index++)
				{
					if (HashDataOld[Index].IsOccupied())
					{
						InsertIntoTable(IndexDataOld[Index], HashDataOld[Index]);
					}
				}
			}

			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentReaders) == 0);
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentWriters) == 0);

			return bIsFoundInMap;
		}

		bool Remove(const KeyType& Key)
		{
			const FHashType HashValue = ComputeHash(Key);
			return RemoveByHash(HashValue, Key);
		}

		inline bool RemoveByElementId(FHashElementId Id)
		{
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentWriters) == 1);
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentReaders) == 1);
			
			bool bIsFoundInMap = false;
			FHashType HashValue = KeyValueData.Hashes[Id.GetIndex()];
			IndexType BucketIndex = ModTableSize(HashValue.AsUInt());
			const IndexType EndBucketIndex = ModTableSize(HashValue.AsUInt() + MaximumDistance + 1);
			do
			{
				if (Id.GetIndex() == IndexData[BucketIndex])
				{
					checkSlow(HashData[BucketIndex] == HashValue);
					KeyValueData.Deallocate(IndexData[BucketIndex]);
					HashData[BucketIndex] = FHashType();
					bIsFoundInMap = true;
					break;
				}

				BucketIndex = ModTableSize(BucketIndex + 1);
			} while (BucketIndex != EndBucketIndex);

			if (bIsFoundInMap && (KeyValueData.Num() * LoadFactorQuotient * 4) < (SizePow2Minus1 * LoadFactorDivisor))
			{
				TArray<IndexType, InlineOneAllocatorType> IndexDataOld = MoveTemp(IndexData);
				TArray<FHashType, InlineOneAllocatorType> HashDataOld = MoveTemp(HashData);
				IndexType OldSizePow2Minus1 = SizePow2Minus1;
				SizePow2Minus1 = SizePow2Minus1 / 2;
				MaximumDistance = 0;
				IndexData.Reserve(SizePow2Minus1 + 1);
				IndexData.AddUninitialized(SizePow2Minus1 + 1);
				HashData.Reserve(SizePow2Minus1 + 1);
				HashData.AddDefaulted(SizePow2Minus1 + 1);

				for (IndexType Index = 0; Index <= OldSizePow2Minus1; Index++)
				{
					if (HashDataOld[Index].IsOccupied())
					{
						InsertIntoTable(IndexDataOld[Index], HashDataOld[Index]);
					}
				}
			}

			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentReaders) == 0);
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentWriters) == 0);

			return bIsFoundInMap;
		}

		void Empty()
		{
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentWriters) == 1);
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentReaders) == 1);

			IndexData.Empty(1); 
			IndexData.AddUninitialized();
			HashData.Empty(1); 
			HashData.AddDefaulted();
			KeyValueData.Empty();
			SizePow2Minus1 = 0;
			MaximumDistance = 0;

			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentReaders) == 0);
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentWriters) == 0);
		}

		void Reserve(SizeType ReserveNum)
		{
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentWriters) == 1);
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedIncrement(&ConcurrentReaders) == 1);

			if (ReserveNum > KeyValueData.Num())
			{
				KeyValueData.Reserve(ReserveNum);

				IndexType NewSizePow2Minus1 = SizePow2Minus1;
				while ((ReserveNum * LoadFactorQuotient) >= (NewSizePow2Minus1 * LoadFactorDivisor))
				{
					NewSizePow2Minus1 = NewSizePow2Minus1 * 2 + 1;
				}

				if (NewSizePow2Minus1 > SizePow2Minus1)
				{
					TArray<IndexType, InlineOneAllocatorType> IndexDataOld = MoveTemp(IndexData);
					TArray<FHashType, InlineOneAllocatorType> HashDataOld = MoveTemp(HashData);
					IndexType OldSizePow2Minus1 = SizePow2Minus1;
					SizePow2Minus1 = NewSizePow2Minus1;
					MaximumDistance = 0;
					IndexData.Reserve(SizePow2Minus1 + 1); 
					IndexData.AddUninitialized(SizePow2Minus1 + 1);
					HashData.Reserve(SizePow2Minus1 + 1);  
					HashData.AddDefaulted(SizePow2Minus1 + 1);

					for (IndexType Index = 0; Index <= OldSizePow2Minus1; Index++)
					{
						if (HashDataOld[Index].IsOccupied())
						{
							InsertIntoTable(IndexDataOld[Index], HashDataOld[Index]);
						}
					}
				}
			}

			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentReaders) == 0);
			CHECK_CONCURRENT_ACCESS(FPlatformAtomics::InterlockedDecrement(&ConcurrentWriters) == 0);
		}

	private:
		FData KeyValueData;

		TArray<IndexType, InlineOneAllocatorType> IndexData;
		TArray<FHashType, InlineOneAllocatorType> HashData;

		IndexType SizePow2Minus1 = 0;
		IndexType MaximumDistance = 0;

#if RUN_HASHTABLE_CONCURENCY_CHECKS
		mutable int ConcurrentReaders = 0;
		mutable int ConcurrentWriters = 0;
#endif
	};
}

template<typename KeyType, typename ValueType, typename Hasher = TDefaultMapHashableKeyFuncs<KeyType, ValueType, false>, typename HashMapAllocator = FDefaultAllocator>
class TRobinHoodHashMap : public RobinHoodHashTable_Private::TRobinHoodHashTable<KeyType, ValueType, Hasher, HashMapAllocator>
{
	using Base = typename RobinHoodHashTable_Private::TRobinHoodHashTable<KeyType, ValueType, Hasher, HashMapAllocator>;
	using IndexType = typename Base::IndexType;
	using FindValueType = typename Base::FindValueType;
	using FindValueTypeConst = typename Base::FindValueTypeConst;

public:
	TRobinHoodHashMap() : Base()
	{
		static_assert(sizeof(Base) == sizeof(TRobinHoodHashMap), "This class should only limit the interface and not implement anything");
	}

	TRobinHoodHashMap(const TRobinHoodHashMap& Other) = default;
	TRobinHoodHashMap& operator=(const TRobinHoodHashMap& Other) = default;
	TRobinHoodHashMap(TRobinHoodHashMap&& Other) = default;
	TRobinHoodHashMap& operator=(TRobinHoodHashMap&& Other) = default;

	FHashElementId FindOrAddIdByHash(FHashType HashValue, const KeyType& Key, const ValueType& Val, bool& bIsAlreadyInMap)
	{
		return Base::FindOrAddIdByHash(HashValue, Key, Val, bIsAlreadyInMap);
	}

	FHashElementId FindOrAddIdByHash(FHashType HashValue, const KeyType& Key, ValueType&& Val, bool& bIsAlreadyInMap)
	{
		return Base::FindOrAddIdByHash(HashValue, Key, MoveTemp(Val), bIsAlreadyInMap);
	}

	FHashElementId FindOrAddIdByHash(FHashType HashValue, KeyType&& Key, const ValueType& Val, bool& bIsAlreadyInMap)
	{
		return Base::FindOrAddIdByHash(HashValue, MoveTemp(Key), Val, bIsAlreadyInMap);
	}

	FHashElementId FindOrAddIdByHash(FHashType HashValue, KeyType&& Key, ValueType&& Val, bool& bIsAlreadyInMap)
	{
		return Base::FindOrAddIdByHash(HashValue, MoveTemp(Key), MoveTemp(Val), bIsAlreadyInMap);
	}

	FHashElementId FindOrAddId(const KeyType& Key, const ValueType& Val, bool& bIsAlreadyInMap)
	{
		return Base::FindOrAddId(Key, Val, bIsAlreadyInMap);
	}

	FHashElementId FindOrAddId(const KeyType& Key, ValueType&& Val, bool& bIsAlreadyInMap)
	{
		return Base::FindOrAddId(Key, MoveTemp(Val), bIsAlreadyInMap);
	}

	FHashElementId FindOrAddId(KeyType&& Key, const ValueType& Val, bool& bIsAlreadyInMap)
	{
		return Base::FindOrAddId(MoveTemp(Key), Val);
	}

	FHashElementId FindOrAddId(KeyType&& Key, ValueType&& Val, bool& bIsAlreadyInMap)
	{
		return Base::FindOrAddId(MoveTemp(Key), MoveTemp(Val), bIsAlreadyInMap);
	}

	FindValueType FindOrAdd(const KeyType& Key, const ValueType& Val, bool& bIsAlreadyInMap)
	{
		return Base::FindOrAdd(Key, Val, bIsAlreadyInMap);
	}

	FindValueType FindOrAdd(const KeyType& Key, ValueType&& Val, bool& bIsAlreadyInMap)
	{
		return Base::FindOrAdd(Key, MoveTemp(Val), bIsAlreadyInMap);
	}

	FindValueType FindOrAdd(KeyType&& Key, const ValueType& Val, bool& bIsAlreadyInMap)
	{
		return Base::FindOrAdd(MoveTemp(Key), Val, bIsAlreadyInMap);
	}

	FindValueType FindOrAdd(KeyType&& Key, ValueType&& Val, bool& bIsAlreadyInMap)
	{
		return Base::FindOrAdd(MoveTemp(Key), MoveTemp(Val), bIsAlreadyInMap);
	}

	FHashElementId FindOrAddIdByHash(FHashType HashValue, const KeyType& Key, const ValueType& Val)
	{
		bool bIsAlreadyInMap;
		return Base::FindOrAddIdByHash(HashValue, Key, Val, bIsAlreadyInMap);
	}

	FHashElementId FindOrAddIdByHash(FHashType HashValue, const KeyType& Key, ValueType&& Val)
	{
		bool bIsAlreadyInMap;
		return Base::FindOrAddIdByHash(HashValue, Key, MoveTemp(Val), bIsAlreadyInMap);
	}

	FHashElementId FindOrAddIdByHash(FHashType HashValue, KeyType&& Key, const ValueType& Val)
	{
		bool bIsAlreadyInMap;
		return Base::FindOrAddIdByHash(HashValue, MoveTemp(Key), Val, bIsAlreadyInMap);
	}

	FHashElementId FindOrAddIdByHash(FHashType HashValue, KeyType&& Key, ValueType&& Val)
	{
		bool bIsAlreadyInMap;
		return Base::FindOrAddIdByHash(HashValue, MoveTemp(Key), MoveTemp(Val), bIsAlreadyInMap);
	}

	FHashElementId FindOrAddId(const KeyType& Key, const ValueType& Val)
	{
		bool bIsAlreadyInMap;
		return Base::FindOrAddId(Key, Val, bIsAlreadyInMap);
	}

	FHashElementId FindOrAddId(const KeyType& Key, ValueType&& Val)
	{
		bool bIsAlreadyInMap;
		return Base::FindOrAddId(Key, MoveTemp(Val), bIsAlreadyInMap);
	}

	FHashElementId FindOrAddId(KeyType&& Key, const ValueType& Val)
	{
		bool bIsAlreadyInMap;
		return Base::FindOrAddId(MoveTemp(Key), Val);
	}

	FHashElementId FindOrAddId(KeyType&& Key, ValueType&& Val)
	{
		bool bIsAlreadyInMap;
		return Base::FindOrAddId(MoveTemp(Key), MoveTemp(Val), bIsAlreadyInMap);
	}

	FindValueType FindOrAdd(const KeyType& Key, const ValueType& Val)
	{
		bool bIsAlreadyInMap;
		return Base::FindOrAdd(Key, Val, bIsAlreadyInMap);
	}

	FindValueType FindOrAdd(const KeyType& Key, ValueType&& Val)
	{
		bool bIsAlreadyInMap;
		return Base::FindOrAdd(Key, MoveTemp(Val), bIsAlreadyInMap);
	}

	FindValueType FindOrAdd(KeyType&& Key, const ValueType& Val)
	{
		bool bIsAlreadyInMap;
		return Base::FindOrAdd(MoveTemp(Key), Val, bIsAlreadyInMap);
	}

	FindValueType FindOrAdd(KeyType&& Key, ValueType&& Val)
	{
		bool bIsAlreadyInMap;
		return Base::FindOrAdd(MoveTemp(Key), MoveTemp(Val), bIsAlreadyInMap);
	}

	FHashElementId UpdateIdByHash(FHashType HashValue, const KeyType& Key, const ValueType& Val, bool& bIsAlreadyInMap)
	{
		return Base::UpdateIdByHash(HashValue, Key, Val, bIsAlreadyInMap);
	}

	FHashElementId UpdateIdByHash(FHashType HashValue, const KeyType& Key, ValueType&& Val, bool& bIsAlreadyInMap)
	{
		return Base::UpdateIdByHash(HashValue, Key, MoveTemp(Val), bIsAlreadyInMap);
	}

	FHashElementId UpdateIdByHash(FHashType HashValue, KeyType&& Key, const ValueType& Val, bool& bIsAlreadyInMap)
	{
		return Base::UpdateIdByHash(HashValue, MoveTemp(Key), Val, bIsAlreadyInMap);
	}

	FHashElementId UpdateIdByHash(FHashType HashValue, KeyType&& Key, ValueType&& Val, bool& bIsAlreadyInMap)
	{
		return Base::UpdateIdByHash(HashValue, MoveTemp(Key), MoveTemp(Val), bIsAlreadyInMap);
	}

	FHashElementId UpdateId(const KeyType& Key, const ValueType& Val, bool& bIsAlreadyInMap)
	{
		return Base::UpdateId(Key, Val, bIsAlreadyInMap);
	}

	FHashElementId UpdateId(const KeyType& Key, ValueType&& Val, bool& bIsAlreadyInMap)
	{
		return Base::UpdateId(Key, MoveTemp(Val), bIsAlreadyInMap);
	}

	FHashElementId UpdateId(KeyType&& Key, const ValueType& Val, bool& bIsAlreadyInMap)
	{
		return Base::UpdateId(MoveTemp(Key), Val);
	}

	FHashElementId UpdateId(KeyType&& Key, ValueType&& Val, bool& bIsAlreadyInMap)
	{
		return Base::UpdateId(MoveTemp(Key), MoveTemp(Val), bIsAlreadyInMap);
	}

	FindValueType Update(const KeyType& Key, const ValueType& Val, bool& bIsAlreadyInMap)
	{
		return Base::Update(Key, Val, bIsAlreadyInMap);
	}

	FindValueType Update(const KeyType& Key, ValueType&& Val, bool& bIsAlreadyInMap)
	{
		return Base::Update(Key, MoveTemp(Val), bIsAlreadyInMap);
	}

	FindValueType Update(KeyType&& Key, const ValueType& Val, bool& bIsAlreadyInMap)
	{
		return Base::Update(MoveTemp(Key), Val, bIsAlreadyInMap);
	}

	FindValueType Update(KeyType&& Key, ValueType&& Val, bool& bIsAlreadyInMap)
	{
		return Base::Update(MoveTemp(Key), MoveTemp(Val), bIsAlreadyInMap);
	}

	FHashElementId UpdateIdByHash(FHashType HashValue, const KeyType& Key, const ValueType& Val)
	{
		bool bIsAlreadyInMap;
		return Base::UpdateIdByHash(HashValue, Key, Val, bIsAlreadyInMap);
	}

	FHashElementId UpdateIdByHash(FHashType HashValue, const KeyType& Key, ValueType&& Val)
	{
		bool bIsAlreadyInMap;
		return Base::UpdateIdByHash(HashValue, Key, MoveTemp(Val), bIsAlreadyInMap);
	}

	FHashElementId UpdateIdByHash(FHashType HashValue, KeyType&& Key, const ValueType& Val)
	{
		bool bIsAlreadyInMap;
		return Base::UpdateIdByHash(HashValue, MoveTemp(Key), Val, bIsAlreadyInMap);
	}

	FHashElementId UpdateIdByHash(FHashType HashValue, KeyType&& Key, ValueType&& Val)
	{
		bool bIsAlreadyInMap;
		return Base::UpdateIdByHash(HashValue, MoveTemp(Key), MoveTemp(Val), bIsAlreadyInMap);
	}

	FHashElementId UpdateId(const KeyType& Key, const ValueType& Val)
	{
		bool bIsAlreadyInMap;
		return Base::UpdateId(Key, Val, bIsAlreadyInMap);
	}

	FHashElementId UpdateId(const KeyType& Key, ValueType&& Val)
	{
		bool bIsAlreadyInMap;
		return Base::UpdateId(Key, MoveTemp(Val), bIsAlreadyInMap);
	}

	FHashElementId UpdateId(KeyType&& Key, const ValueType& Val)
	{
		bool bIsAlreadyInMap;
		return Base::UpdateId(MoveTemp(Key), Val);
	}

	FHashElementId UpdateId(KeyType&& Key, ValueType&& Val)
	{
		bool bIsAlreadyInMap;
		return Base::UpdateId(MoveTemp(Key), MoveTemp(Val), bIsAlreadyInMap);
	}

	FindValueType Update(const KeyType& Key, const ValueType& Val)
	{
		bool bIsAlreadyInMap;
		return Base::Update(Key, Val, bIsAlreadyInMap);
	}

	FindValueType Update(const KeyType& Key, ValueType&& Val)
	{
		bool bIsAlreadyInMap;
		return Base::Update(Key, MoveTemp(Val), bIsAlreadyInMap);
	}

	FindValueType Update(KeyType&& Key, const ValueType& Val)
	{
		bool bIsAlreadyInMap;
		return Base::Update(MoveTemp(Key), Val, bIsAlreadyInMap);
	}

	FindValueType Update(KeyType&& Key, ValueType&& Val)
	{
		bool bIsAlreadyInMap;
		return Base::Update(MoveTemp(Key), MoveTemp(Val), bIsAlreadyInMap);
	}
};

template<typename KeyType, typename Hasher = DefaultKeyFuncs<KeyType, false>, typename HashMapAllocator = FDefaultAllocator>
class TRobinHoodHashSet : public RobinHoodHashTable_Private::TRobinHoodHashTable<KeyType, RobinHoodHashTable_Private::FUnitType, Hasher, HashMapAllocator>
{
	using Unit = RobinHoodHashTable_Private::FUnitType;
	using Base = typename RobinHoodHashTable_Private::TRobinHoodHashTable<KeyType, Unit, Hasher, HashMapAllocator>;
	using IndexType = typename Base::IndexType;
	using FindValueType = typename Base::FindValueType;
	using FindValueTypeConst = typename Base::FindValueTypeConst;

public:
	TRobinHoodHashSet() : Base()
	{
		static_assert(sizeof(Base) == sizeof(TRobinHoodHashSet), "This class should only limit the interface and not implement anything");
	}

	TRobinHoodHashSet(const TRobinHoodHashSet& Other) = default;
	TRobinHoodHashSet& operator=(const TRobinHoodHashSet& Other) = default;
	TRobinHoodHashSet(TRobinHoodHashSet&& Other) = default;
	TRobinHoodHashSet& operator=(TRobinHoodHashSet&& Other) = default;

	FHashElementId FindOrAddIdByHash(FHashType HashValue, const KeyType& Key, bool& bIsAlreadyInSet)
	{
		return Base::FindOrAddIdByHash(HashValue, Key, Unit(), bIsAlreadyInSet);
	}

	FHashElementId FindOrAddIdByHash(FHashType HashValue, KeyType&& Key, bool& bIsAlreadyInSet)
	{
		return Base::FindOrAddIdByHash(HashValue, MoveTemp(Key), Unit(), bIsAlreadyInSet);
	}

	FHashElementId FindOrAddId(const KeyType& Key, bool& bIsAlreadyInSet)
	{
		return Base::FindOrAddId(Key, Unit(), bIsAlreadyInSet);
	}

	FHashElementId FindOrAddId(KeyType&& Key, bool& bIsAlreadyInSet)
	{
		return Base::FindOrAddId(MoveTemp(Key), Unit(), bIsAlreadyInSet);
	}

	FindValueType FindOrAdd(const KeyType& Key, bool& bIsAlreadyInSet)
	{
		return Base::FindOrAdd(Key, Unit(), bIsAlreadyInSet);
	}

	FindValueType FindOrAdd(KeyType&& Key, bool& bIsAlreadyInSet)
	{
		return Base::FindOrAdd(MoveTemp(Key), Unit(), bIsAlreadyInSet);
	}

	FHashElementId FindOrAddIdByHash(FHashType HashValue, const KeyType& Key)
	{
		bool bIsAlreadyInSet;
		return Base::FindOrAddIdByHash(HashValue, Key, Unit(), bIsAlreadyInSet);
	}

	FHashElementId FindOrAddIdByHash(FHashType HashValue, KeyType&& Key)
	{
		bool bIsAlreadyInSet;
		return Base::FindOrAddIdByHash(HashValue, MoveTemp(Key), Unit(), bIsAlreadyInSet);
	}

	FHashElementId FindOrAddId(const KeyType& Key)
	{
		bool bIsAlreadyInSet;
		return Base::FindOrAddId(Key, Unit(), bIsAlreadyInSet);
	}

	FHashElementId FindOrAddId(KeyType&& Key)
	{
		bool bIsAlreadyInSet;
		return Base::FindOrAddId(MoveTemp(Key), Unit(), bIsAlreadyInSet);
	}

	FindValueType FindOrAdd(const KeyType& Key)
	{
		bool bIsAlreadyInSet;
		return Base::FindOrAdd(Key, Unit(), bIsAlreadyInSet);
	}

	FindValueType FindOrAdd(KeyType&& Key)
	{
		bool bIsAlreadyInSet;
		return Base::FindOrAdd(MoveTemp(Key), Unit(), bIsAlreadyInSet);
	}
};

};

#undef CHECK_CONCURRENT_ACCESS
#undef RUN_HASHTABLE_CONCURENCY_CHECKS