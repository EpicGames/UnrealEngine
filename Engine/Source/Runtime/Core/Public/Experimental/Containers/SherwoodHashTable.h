// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/UnrealTemplate.h"

namespace Experimental
{

template<typename KeyType, typename ValueType>
struct TSherwoodHashKeyFuncs
{
	typedef typename TTypeTraits<KeyType>::ConstPointerType KeyInitType;

	static FORCEINLINE bool Matches(KeyInitType A, KeyInitType B)
	{
		return A == B;
	}
	static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key);
	}
};

namespace TSherwoodHashTable_Private
{

/**
* Implementation of Robin Hood hash table based on sherwood_v3_table by Malte Skarupke.
* Good for small keys and values. If key is already a high quality hash, then identity hash function should be used.
* Current limitations:
*   - Requires key and value to be trivial types.
*   - Does not allow move or copy.
*   - Does not support custom allocators.
*/
template <typename KeyType, typename ValueType, typename KeyFuncs = TSherwoodHashKeyFuncs<KeyType, ValueType>>
struct TSherwoodHashTable
{
	using HashType = uint32; // TODO: perhaps we could deduce type of KeyFuncs::GetKeyHash() instead

	// TSherwoodHashTable can be used to implement a set or a map.
	// In map mode we allocate memory for keys and values, but in set mode we only allocate keys.
	static constexpr bool bIsMap = !std::is_same_v<ValueType, FNoopStruct>;

	// Minimum probing distance when searching for an entry slot.
	static constexpr uint32 MinNumLookups = 4;

	// Smallest capacity of non-empty container.
	static constexpr uint32 MinNumSlots = 4;

	// Ratio between number of stored elements and allocated capacity beyond which the container will be grown (doubled in size).
	static constexpr float MaxLoadFactor = 0.9f;
	static_assert(MaxLoadFactor >= 0.5 && MaxLoadFactor <= 0.9, "MaxLoadFactor must be in range [0.5 .. 0.9]");

	// NOTE: Non-trivial type support, move and copy ops are not implemented yet, but can be.
	static_assert(TIsTrivial<KeyType>::Value, "Key is expected to be a trivial type.");
	static_assert(TIsTrivial<ValueType>::Value || !bIsMap, "Value is expected to be a trivial type.");

	TSherwoodHashTable() = default;
	TSherwoodHashTable(const TSherwoodHashTable&) = delete;
	TSherwoodHashTable(TSherwoodHashTable&&) = delete;
	TSherwoodHashTable& operator = (const TSherwoodHashTable&) = delete;
	TSherwoodHashTable& operator = (TSherwoodHashTable&&) = delete;
	~TSherwoodHashTable()
	{
		Empty();
	}

	template <typename T>
	static T* AllocateUninitialized(uint32 Count)
	{
		return (T*)FMemory::Malloc(sizeof(T) * Count);
	}

	static void Deallocate(void* Ptr)
	{
		FMemory::Free(Ptr);
	}

	struct FData
	{
		bool HasValue(uint32 i) const
		{
			return Distances[i] >= 0;
		}

		bool IsEmpty(uint32 i) const
		{
			return Distances[i] < 0;
		}

		void AddAt(uint32 i, int8 InDistance, KeyType InKey, ValueType InValue)
		{
			Keys[i] = InKey;
			if (bIsMap)
			{ 
				Values[i] = InValue; 
			}
			Distances[i] = InDistance;
		}

		bool IsValid() const { return Distances != nullptr; }

		uint32 AllocatedCount = 0;
		int8* Distances = nullptr;
		KeyType* Keys = nullptr;
		ValueType* Values = nullptr;
	};

	FData CurrentData;
	uint32 NumSlotsMinusOne = 0;
	int8 MaxLookups = 0;
	int32 NumElements = 0;

	static void DeallocateData(FData& Data)
	{
		if (Data.IsValid())
		{
			// Assumes that individual elements are already destroyed.

			Deallocate(Data.Distances);
			Deallocate(Data.Keys);
			Deallocate(Data.Values);
		}
	}

	void Reset()
	{
		if (CurrentData.IsValid())
		{
			FMemory::Memset(CurrentData.Distances, uint8(-1), CurrentData.AllocatedCount);
		}
		NumElements = 0;
	}

	void Empty()
	{
		DeallocateData(CurrentData);

		CurrentData = FData();
		NumSlotsMinusOne = 0;
		MaxLookups = 0;
		NumElements = 0;
	}

	uint32 NumSlots() const
	{
		return NumSlotsMinusOne ? NumSlotsMinusOne + 1 : 0;
	}

	static uint8 ComputeMaxLookups(uint32 InNumSlots)
	{
		uint32 Desired = (32-FMath::CountLeadingZeros(InNumSlots)); // FloorLog2
		return uint8(FMath::Max(MinNumLookups, Desired));
	}

	static FData AllocateData(uint32 Count)
	{
		FData Result;

		Result.AllocatedCount = Count;
		Result.Distances = AllocateUninitialized<int8>(Count);
		Result.Keys = AllocateUninitialized<KeyType>(Count);
		if (bIsMap)
		{
			Result.Values = AllocateUninitialized<ValueType>(Count);
		}

		FMemory::Memset(Result.Distances, uint8(-1), Count);

		return Result;
	}

	FORCEINLINE_DEBUGGABLE TTuple<const KeyType*, ValueType*> Find(KeyType Key) const
	{
		if (CurrentData.IsValid())
		{
			int8 Distance = 0;
			uint32 Cursor = KeyFuncs::GetKeyHash(Key) & NumSlotsMinusOne; // Number of slots is always Pow2
			for (; CurrentData.Distances[Cursor] >= Distance; ++Cursor, ++Distance)
			{
				const KeyType& KeyAtCursor = CurrentData.Keys[Cursor];
				if (KeyFuncs::Matches(Key, KeyAtCursor))
				{
					return TTuple<const KeyType*, ValueType*>(&KeyAtCursor, bIsMap ? &CurrentData.Values[Cursor] : nullptr);
				}
			}
		}

		return TTuple<const KeyType*, ValueType*>(nullptr, nullptr);
	}

	FORCEINLINE_DEBUGGABLE ValueType* FindOrAdd(KeyType Key, ValueType Value, bool* bIsAlreadyInContainerPtr = nullptr)
	{
		return FindOrAddByHash(Key, KeyFuncs::GetKeyHash(Key), Value, bIsAlreadyInContainerPtr);
	}

	FORCEINLINE_DEBUGGABLE ValueType* FindOrAddByHash(KeyType Key, HashType Hash, ValueType Value, bool* bIsAlreadyInContainerPtr = nullptr)
	{
		uint32 Cursor = Hash & NumSlotsMinusOne; // Number of slots is always Pow2
		int8 Distance = 0;

		if (CurrentData.IsValid())
		{
			for (; CurrentData.Distances[Cursor] >= Distance; ++Cursor, ++Distance)
			{
				if (KeyFuncs::Matches(Key, CurrentData.Keys[Cursor]))
				{
					if (bIsAlreadyInContainerPtr)
					{
						*bIsAlreadyInContainerPtr = true;
					}
					return bIsMap ? &CurrentData.Values[Cursor] : nullptr;
				}
			}
		}

		if (bIsAlreadyInContainerPtr)
		{
			*bIsAlreadyInContainerPtr = false;
		}

		return Add(Distance, CurrentData, Cursor, Key, Hash, Value);
	}

	FORCENOINLINE ValueType* Add(int8 Distance, FData& Data, uint32 Cursor, KeyType Key, HashType Hash, ValueType Value)
	{
		if (Distance == MaxLookups || NumElements + 1 > int32(float(NumSlotsMinusOne + 1) * MaxLoadFactor))
		{
			Grow();
			return FindOrAddByHash(Key, Hash, Value);
		}
		else if (Data.IsEmpty(Cursor))
		{
			Data.AddAt(Cursor, Distance, Key, Value);
			++NumElements;
			return bIsMap ? &Data.Values[Cursor] : nullptr;
		}

		Swap(Distance, Data.Distances[Cursor]);
		Swap(Key, Data.Keys[Cursor]);
		if (bIsMap)
		{
			Swap(Value, Data.Values[Cursor]);
		}

		const uint32 ResultCursor = Cursor;
		++Distance;
		++Cursor;
		for (;;)
		{
			if (Data.IsEmpty(Cursor))
			{
				Data.AddAt(Cursor, Distance, Key, Value);
				++NumElements;
				return bIsMap ? &(Data.Values[ResultCursor]) : nullptr;
			}
			else if (Data.Distances[Cursor] < Distance)
			{
				Swap(Distance, Data.Distances[Cursor]);
				Swap(Key, Data.Keys[Cursor]);
				if (bIsMap)
				{
					Swap(Value, Data.Values[Cursor]);
				}
				++Distance;
			}
			else
			{
				++Distance;
				if (Distance == MaxLookups)
				{
					Swap(Key, Data.Keys[ResultCursor]);
					if (bIsMap)
					{
						Swap(Value, Data.Values[ResultCursor]);
					}
					Grow();
					return FindOrAdd(Key, Value);
				}
			}
			++Cursor;
		}
	}

	void Rehash(uint32 DesiredNumSlots)
	{
		DesiredNumSlots = FMath::Max(DesiredNumSlots, uint32(FMath::CeilToInt(float(NumElements) / MaxLoadFactor)));
		if (DesiredNumSlots == 0)
		{
			Empty();
			return;
		}

		DesiredNumSlots = FMath::RoundUpToPowerOfTwo(DesiredNumSlots);

		if (DesiredNumSlots == NumSlots())
		{
			return;
		}

		const uint8 NewMaxLookups = ComputeMaxLookups(DesiredNumSlots);

		FData OldData = CurrentData;
		const uint32 OldNumSlotsMinusOne = NumSlotsMinusOne;
		const uint32 OldMaxLookups = MaxLookups;

		CurrentData = AllocateData(DesiredNumSlots + NewMaxLookups);

		NumSlotsMinusOne = DesiredNumSlots - 1;
		MaxLookups = NewMaxLookups;
		NumElements = 0;

		for (uint32 Index = 0, End = OldNumSlotsMinusOne + OldMaxLookups; Index != End; ++Index)
		{
			if (OldData.HasValue(Index))
			{
				if (bIsMap)
				{
					FindOrAdd(OldData.Keys[Index], OldData.Values[Index]);
				}
				else
				{
					FindOrAdd(OldData.Keys[Index], ValueType{});
				}
			}
		}

		DeallocateData(OldData);
	}

	void Grow()
	{
		uint32 NextNumSlots = FMath::Max<uint32>(MinNumSlots, 2u * NumSlots());
		Rehash(NextNumSlots);
	}

	void Reserve(uint32 DesiredNumElements)
	{
		uint32 DesiredNumSlots = FMath::Max<uint32>(DesiredNumElements, uint32(FMath::CeilToInt(float(DesiredNumElements) / MaxLoadFactor)));
		if (DesiredNumSlots > NumSlots())
		{
			Rehash(DesiredNumSlots);
		}
	}
};

} // namespace TSherwoodHashTable_Private

template <typename KeyType, typename ValueType, typename KeyFuncs = TSherwoodHashKeyFuncs<KeyType, ValueType>>
struct TSherwoodMap
{
	ValueType& FindOrAdd(KeyType Key, ValueType Value)
	{
		return *Table.FindOrAdd(Key, Value);
	}

	ValueType* Find(KeyType Key)
	{
		return Table.Find(Key).Value;
	}

	const ValueType* Find(KeyType Key) const
	{
		return Table.Find(Key).Value;
	}

	int32 Num() const
	{
		return Table.NumElements;
	}

	void Empty()
	{
		Table.Empty();
	}

	void Reset()
	{
		Table.Reset();
	}

	void Reserve(uint32 DesiredNumElements)
	{
		Table.Reserve(DesiredNumElements);
	}

private:
	TSherwoodHashTable_Private::TSherwoodHashTable<KeyType, ValueType, KeyFuncs> Table;
};

template <typename KeyType, typename KeyFuncs = TSherwoodHashKeyFuncs<KeyType, FNoopStruct>>
struct TSherwoodSet
{
	void Add(KeyType Key, bool* bIsAlreadyInSetPtr = nullptr)
	{
		Table.FindOrAdd(Key, FNoopStruct{}, bIsAlreadyInSetPtr);
	}

	const KeyType* Find(KeyType Key) const
	{
		return Table.Find(Key).Key;
	}

	int32 Num() const
	{
		return Table.NumElements;
	}

	void Empty()
	{
		Table.Empty();
	}

	void Reset()
	{
		Table.Reset();
	}

	void Reserve(uint32 DesiredNumElements)
	{
		Table.Reserve(DesiredNumElements);
	}

private:
	TSherwoodHashTable_Private::TSherwoodHashTable<KeyType, FNoopStruct, KeyFuncs> Table;
};

} // namespace Experimental
