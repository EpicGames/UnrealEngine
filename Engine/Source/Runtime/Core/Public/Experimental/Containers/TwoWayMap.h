// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"

/** Statically allocates two maps from the same array of pairs */
template<typename ClassT, typename FirstType, typename SecondType>
struct TTwoWayMap
{
	typedef TArray<TPair<FirstType, SecondType>> InitListType;

	static const TMap<FirstType, SecondType>& GetFirstToSecond()
	{
		InitIfNeeded();
		return FirstToSecond;
	}

	static const TMap<SecondType, FirstType>& GetSecondToFirst()
	{
		InitIfNeeded();
		return SecondToFirst;
	}
	static void InitIfNeeded()
	{
		if (FirstToSecond.Num() == 0 || SecondToFirst.Num() == 0)
		{
			FirstToSecond.Reserve(ClassT::PairDefinitions.Num());
			SecondToFirst.Reserve(ClassT::PairDefinitions.Num());

			for (const TPair<FirstType, SecondType>& Pair : ClassT::PairDefinitions)
			{
				FirstToSecond.Emplace(Pair.Key, Pair.Value);
				SecondToFirst.Emplace(Pair.Value, Pair.Key);
			}

			ClassT::PairDefinitions.Reset();
		}
	}

protected:
	inline static TMap<FirstType, SecondType> FirstToSecond = TMap<FirstType, SecondType>();
	inline static TMap<SecondType, FirstType> SecondToFirst = TMap<SecondType, FirstType>();
};

template<typename DomainType, typename RangeType>
struct TBijectionMaps
{
protected:
	struct FPairValue
	{
		const DomainType X;
		const RangeType Y;
	};

public:
	TBijectionMaps() = delete;

	constexpr TBijectionMaps(const std::initializer_list<FPairValue>& InitList)
	{
		// TODO: Run compile time check to make sure all keys and values are unique
		for (const FPairValue& Pair : InitList)
		{
			if (ensureAlwaysMsgf(!Image.Contains(Pair.X), TEXT("This cannot be a bijection if there is a duplicate domain entry")))
			{
				Image.Emplace(Pair.X, Pair.Y);
			}
			if (ensureAlwaysMsgf(!PreImage.Contains(Pair.Y), TEXT("This cannot be a bijection if there is a duplicate range entry")))
			{
				PreImage.Emplace(Pair.Y, Pair.X);
			}
		}
	}

	constexpr const DomainType* Find(const RangeType& InValue) const
	{
		return PreImage.Find(InValue);
	}

	constexpr const DomainType& FindChecked(const RangeType& InValue) const
	{
		return PreImage[InValue];
	}

	constexpr const RangeType* Find(const DomainType& InValue) const
	{
		return Image.Find(InValue);
	}

	constexpr const RangeType& FindChecked(const DomainType& InValue) const
	{
		return Image[InValue];
	}

private:
	TMap<DomainType, RangeType> Image = TMap<DomainType, RangeType>();
	TMap<RangeType, DomainType> PreImage = TMap<RangeType, DomainType>();
};