// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Traits/ElementType.h"

#include <utility>

namespace UE::Graph
{

typedef int32 FVertex;
// Valid vertices are in the range 0 <= vertex < MAX_int32.
// This ensures that a edge list that contains an edge to every vertex from 0 to MaxVertex inclusive in a graph can be stored in
// a TArray or TArrayView.
inline constexpr FVertex MaxVertex = static_cast<FVertex>(MAX_int32-1);
inline constexpr FVertex InvalidVertex = static_cast<FVertex>(INDEX_NONE);

/**
 * Convenience structure for storing a graph in edge list form.
 * EdgeLists[i] is a TConstArrayView of edges from Vertex i.
 * Each element of EdgeList[i] is a an FVertex j such that (i, j) is an edge in the graph.
 */
struct FGraph
{
	// Singular buffer which each element of EdgeLists is a view into.
	TArray64<FVertex> Buffer;
	// Each member of EdgeLists is a slice of Buffer.
	TArray<TConstArrayView<FVertex>> EdgeLists;

	FGraph() = default;
	// Copy construction not implemented as buffers may be very large and expensive to copy
	FGraph(const FGraph&) = delete;
	FGraph& operator=(const FGraph&) = delete;
	FGraph(FGraph&&) = default;
	FGraph& operator=(FGraph&&) = default;

	SIZE_T GetAllocatedSize() const 
	{
		return Buffer.GetAllocatedSize() + EdgeLists.GetAllocatedSize();
	}
};

/** 
 * Represents a mapping between two graphs where each vertex in a source graph maps to multiple vertices in a target graph.
 * The length of Mapping is the number of vertices in the source graph and the contents of each mapping are vertices in the target graph.
 * Buffer stores the contents of all array views in Mapping.
 * e.g. a mapping between the vertices in a condensation graph and the original cyclic graph it was generated from.
 */
struct FMappingOneToMany 
{
	TArray64<FVertex> Buffer;
	TArray<TConstArrayView<FVertex>> Mapping;

	FMappingOneToMany() = default;
	// Copy construction not implemented as buffers may be very large and expensive to copy
	FMappingOneToMany(const FMappingOneToMany&) = delete;
	FMappingOneToMany& operator=(const FMappingOneToMany&) = delete;
	FMappingOneToMany(FMappingOneToMany&&) = default;
	FMappingOneToMany& operator=(FMappingOneToMany&&) = default;
	
	SIZE_T GetAllocatedSize() const 
	{
		return Buffer.GetAllocatedSize() + Mapping.GetAllocatedSize();
	}
};

/** 
 * Represents a mapping between two graphs where each vertex in a source graph maps to exactly one vertex in a target graph.
 * The mapping is not bidirectionally one-to-one.
 * The length of Mapping is the number of vertices in the source graph and each value is a vertex in the target graph.
 * e.g. a mapping from the vertices in a cyclic graph to the vertices in its condensation graph.
 */
struct FMappingManyToOne
{
	TArray<FVertex> Mapping;

	FMappingManyToOne() = default;
	FMappingManyToOne(const FMappingManyToOne&) = default;
	FMappingManyToOne& operator=(const FMappingManyToOne&) = default;
	FMappingManyToOne(FMappingManyToOne&&) = default;
	FMappingManyToOne& operator=(FMappingManyToOne&&) = default;

	SIZE_T GetAllocatedSize() const 
	{
		return Mapping.GetAllocatedSize();
	}
};

enum class EConvertToGraphOptions
{
	None = 0,
	/** The output buffer will be shrunk to reduce memory to minimum required. */
	Shrink = 1 << 0,
};
ENUM_CLASS_FLAGS(EConvertToGraphOptions);

/**
 * Convert an array of Keys and a function that returns the directed edges from each Key into the Normalized Graph
 * Form. Normalized Graph Form is an implicit set of vertices identified as integers from 0 to N - 1, and an array of
 * arrayviews of edges from each vertex. The Normalized Graph form has two structures: The buffer which contains
 * the memory storage for the graph and the graph itself which is the array of arrayviews of edges. The length of the
 * array in the graph defines the number of vertices.
 * 
 * @param UniqueKeys A range with element type KeyType. KeyType must support GetTypeHash and copy+move constructors.
 *        KeyType being pointertype is recommended. Keys in the range must be unique. The ith element of the range
 *        will correspond to a vertex i in the OutGraph. UniqueKeys must have length < MAX_int32.
 * @param GetKeyEdges A callable with prototype that is one of
 *            RangeType<KeyType> GetKeyEdges(const KeyType& Key);
 *            const RangeType<KeyType>& GetKeyEdges(const KeyType& Key);
 *        It must return the TargetKeys that are pointed to from the directed edges from Key. TargetKeys that are 
 *        not elements of UniqueKeys will be discarded. TargetKeys equal to Key will also be discarded. 
 *        RangeType must support ranged-for (begin() and end()).
 * @param Options for the conversion, @see EConvertToGraphOptions.
 * @return Output value that holds the graph. See @FGraph for a description of the format. Vertex i in the graph 
 *         corresponds to UniqueKeys[i].
 */
template <typename RangeType, typename GetKeyEdgesType>
inline FGraph ConvertToGraph(
	const RangeType& UniqueKeys,
	GetKeyEdgesType GetKeyEdges,
	EConvertToGraphOptions Options = EConvertToGraphOptions::None)
{
	using KeyType = TElementType_T<RangeType>;

	// Map Elements to vertices in VertexOfKey
	int32 NumVertices = GetNum(UniqueKeys);
	TMap<KeyType, FVertex> VertexOfKey;
	VertexOfKey.Reserve(NumVertices);
	FVertex Vertex = 0;
	for (const KeyType& Key : UniqueKeys)
	{
		FVertex& ExistingHandle = VertexOfKey.FindOrAdd(Key, InvalidVertex);
		check(ExistingHandle == InvalidVertex);
		if (ExistingHandle == InvalidVertex)
		{
			ExistingHandle = Vertex++;
		}
	}

	FGraph OutGraph;
	OutGraph.Buffer.Reset();
	if (EnumHasAnyFlags(Options, EConvertToGraphOptions::Shrink))
	{
		OutGraph.EdgeLists.Empty(NumVertices);
	}
	else
	{
		OutGraph.EdgeLists.Reset(NumVertices);
	}

	// Temporary base pointer for edge lists until the data pointer of OutGraph.Buffer is fixed. 
	FVertex* EdgeListBase = nullptr;
	// Call GetKeyEdges on each Element, map its returnvalue to Vertices, normalize them, and store them in EdgeOffsets
	Vertex = 0;
	for (const KeyType& Element : UniqueKeys)
	{
		int64 InitialOffset = OutGraph.Buffer.Num();
		for (const KeyType& Dependency : Invoke(GetKeyEdges, Element))
		{
			FVertex* TargetVertex = VertexOfKey.Find(Dependency);
			if (!TargetVertex)
			{
				continue;
			}

			// Normalize Step 1: remove edges to self
			if (*TargetVertex == Vertex)
			{
				continue;
			}
			OutGraph.Buffer.Add(*TargetVertex);
		}

		// Normalize Step 2: Sort
		TArrayView64<FVertex> VertexEdges = TArrayView64<FVertex>(OutGraph.Buffer).Mid(InitialOffset);
		Algo::Sort(VertexEdges);

		// Normalize Step 3: Remove duplicates
		VertexEdges = VertexEdges.Left(Algo::Unique(VertexEdges));
		OutGraph.Buffer.SetNum(InitialOffset + VertexEdges.Num(), EAllowShrinking::No);

		// Store the vertex's offset into GraphBuffer, for later fixup
		// Edge list is guaranteed to fit within 32 bits after removal of duplicates
		OutGraph.EdgeLists.Emplace(EdgeListBase + InitialOffset, static_cast<int32>(VertexEdges.Num())); //-V769

		++Vertex;
	}
	if (EnumHasAnyFlags(Options, EConvertToGraphOptions::Shrink))
	{
		OutGraph.Buffer.Shrink();
	}

	// Correct data pointers in edge lists 
	FVertex* NewBase = OutGraph.Buffer.GetData();
	for (TConstArrayView<FVertex>& EdgeList : OutGraph.EdgeLists)
	{
		EdgeList = TConstArrayView<FVertex>(NewBase + (EdgeList.GetData() - EdgeListBase), EdgeList.Num());
	}
	return OutGraph;
}

/**
 * Convert an array of separately allocated edge ranges into a single buffer shared by all vertices and an edge graph
 * of arrayviews into that buffer. No overlapping is done; every input edge range is reproduced in the output buffer.
 * No validation or normalization is performed; the final edge lists may contain integers which are not valid indices 
 * into the graph, duplicates, etc. 
 * 
 * @param Graph ArrayView of N vertices; each element is interpreted by the ProjectionType to return a range of edges
 *        for that vertex.
 * @param Proj Returns the edges for Graph[n]. The prototype is
 *            // Input is either const& or by value. Output can be const&, &&, or by value.
 *            EdgeRangeType Proj(MultiBufferRangeType VertexEdges)
 *        EdgeRangeType must support GetData and GetNum.
 * @param Options for the conversion, @see EConvertToGraphOptions.
 */
template <typename RangeType, typename ProjectionType>
inline FGraph ConvertToSingleBufferGraph(
	RangeType&& Graph,
	ProjectionType Proj,
	EConvertToGraphOptions Options = EConvertToGraphOptions::None)
{
	typedef TElementType_T<RangeType> InEdgeRangeType;
	typedef decltype(Proj(std::declval<const InEdgeRangeType&>())) ProjectedEdgeRangeType;

	int32 NumVertices = Graph.Num();
	int64 NumEdges = 0;
	for (const InEdgeRangeType& VertexEdges : Graph)
	{
		NumEdges += GetNum(Proj(VertexEdges));
	}

	FGraph OutGraph;
	if (EnumHasAnyFlags(Options, EConvertToGraphOptions::Shrink))
	{
		OutGraph.Buffer.Empty(NumEdges);
		OutGraph.EdgeLists.Empty(NumVertices);
	}
	else
	{
		OutGraph.Buffer.Reset(NumEdges);
		OutGraph.EdgeLists.Reset(NumVertices);
	}

	FVertex* OutBufferData = OutGraph.Buffer.GetData();
	for (const InEdgeRangeType& InVertexEdges : Graph)
	{
		ProjectedEdgeRangeType VertexEdges = Proj(InVertexEdges);
		int32 NumVertexEdges = GetNum(VertexEdges);
		OutGraph.EdgeLists.Emplace(OutBufferData + OutGraph.Buffer.Num(), NumVertexEdges);
		OutGraph.Buffer.Append(GetData(VertexEdges), NumVertexEdges);
	}
	check(OutGraph.Buffer.GetData() == OutBufferData); // We have arrayviews into OutBufferData
	return OutGraph;	
}

/**
 * Convert an array of separately allocated edge ranges into a single buffer shared by all vertices and an edge graph
 * of arrayviews into that buffer. No overlapping is done; every input edge range is reproduced in the output buffer.
 * This version requires MultiBufferRange type to be a range type; see other prototype to provide a Projection.
 * 
 * @param Graph ArrayView of N vertices; each element is interpreted by the ProjectionType to return a range of edges
 *        for that vertex.
 * @param Options for the conversion, @see EConvertToGraphOptions.
 */
template <typename RangeType>
inline FGraph ConvertToSingleBufferGraph(RangeType Graph, 
	EConvertToGraphOptions Options = EConvertToGraphOptions::None)
{
	return ConvertToSingleBufferGraph(Graph, FIdentityFunctor(), Options);
}

/**
 * Return a new buffer and graph with the same vertices but with each edge reversed. If and only if (i,j) is an edge
 * in the input Graph, (j,i) is an edge in the output TransposeGraph.
 * 
 * @param Graph A standard GraphConvert-format Graph. @see ConvertToGraph for a description of the format.
 * 		  Entries in these edges lists must themselves be valid indices into Graph.
 * @param Options for the conversion, @see EConvertToGraphOptions.
 */
CORE_API FGraph ConstructTransposeGraph(
	TConstArrayView<TConstArrayView<FVertex>> Graph,
	EConvertToGraphOptions Options = EConvertToGraphOptions::None);


/**
 * Return a new graph where cycles in the input graph have been replaced by a single vertex. The new graph is 
 * topologically sorted from root to leaf. If the input graph has no cycles, the function returns false and the 
 * OutGraph is reset to empty.
 * 
 * After conversion you can iterate over the OutCondensationVertexToInputVertex mapping to go through the map topologically.
 *
 * @param Graph The input graph, may contain cycles. @see ConvertToGraph for a description of the format.
 * @param OutGraph Output value that holds the graph. 
 *        It has no cycles and the vertices are topologically sorted from root to leaf.
 * @param OutCondensationVertexToInputVertex Output value, optional, can be null. If non-null, it is populated with a 
 *        mapping between the vertices in the condensation and the vertices in the input graph. See @FMappingOneToMany 
 *        for a description of the format. 
 *        Vertices in the condensation which correspond to input vertices which were not in a cycle map to single element.
 *        This value is populated even if the function returns false.
 * @param OutInputVertexToCondensationVertex Output value, optional, can be null. If non-null, it is populated with a
 * 		  mapping from vertices in the input graph to vertices in the condensation. See @FMappingManyToOne for a description
 *        of the format.
 * @param Options for the conversion, @see EConvertToGraphOptions.
 *
 * @return True if the input graph had any cycles, false otherwise. If false is returned, OutGraph is reset to empty,
 * 		   but the other output variables are still populated if present.
 */
CORE_API bool TryConstructCondensationGraph(
	TConstArrayView<TConstArrayView<FVertex>> Graph,
	FGraph& OutGraph,
	FMappingOneToMany* OutCondensationVertexToInputVertex,
	FMappingManyToOne* OutInputVertexToCondensationVertex,
	EConvertToGraphOptions Options = EConvertToGraphOptions::None);

/**
 * Construct the TransposeGraph (see @ConstructTransposeGraph), but only include edges in the original graph
 * from the given vertices (in the transpose graph, edges will be present to the given vertices).
 * Additionally, limit the number of edges in the returned graph, and report which InVertices were added.
 * This is used to create a partial ReachedBy graph from a large Reachability graph.
 * 
 * @param InVertices Vertices in the input graph, edges from which will be included in the output.
 *        This list will be modified: vertices will be sorted from largest number of edges to smallest.
 * @param OutInputVerticesPresentInOutputGraph Output value. Populated with a list of which vertices from 
 *        InVertices were used when constructing the output graph.
 */
CORE_API FGraph ConstructPartialTransposeGraph(
	TConstArrayView<TConstArrayView<FVertex>> Graph,
	TArrayView<FVertex> InVertices,
	int64 MaxOutGraphEdges,
	TArray<FVertex>& OutInputVerticesPresentInOutputGraph);
}