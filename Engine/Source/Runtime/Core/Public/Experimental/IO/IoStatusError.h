// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Experimental/UnifiedError/UnifiedError.h"
#include "IO/IoStatus.h"

#define UE_API CORE_API

class FIoStatus;

UE_DECLARE_ERROR_MODULE(UE_API, UE::IoStore);

UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::Unknown, 				Unknown, 				"EIoErrorCode::Unknown {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::InvalidCode, 			InvalidCode, 			"EIoErrorCode::InvalidCode {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::Cancelled, 				Cancelled, 				"EIoErrorCode::Cancelled {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::FileOpenFailed, 		FileOpenFailed, 		"EIoErrorCode::FileOpenFailed {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::FileNotOpen, 			FileNotOpen, 			"EIoErrorCode::FileNotOpen {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::ReadError, 				ReadError, 				"EIoErrorCode::ReadError {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::WriteError, 			WriteError, 			"EIoErrorCode::WriteError {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::NotFound, 				NotFound, 				"EIoErrorCode::NotFound {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::CorruptToc, 			CorruptToc, 			"EIoErrorCode::CorruptToc {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::UnknownChunkID, 		UnknownChunkID, 		"EIoErrorCode::UnknownChunkID {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::InvalidParameter, 		InvalidParameter, 		"EIoErrorCode::InvalidParameter {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::SignatureError, 		SignatureError, 		"EIoErrorCode::SignatureError {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::InvalidEncryptionKey, 	InvalidEncryptionKey, 	"EIoErrorCode::InvalidEncryptionKey {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::CompressionError, 		CompressionError, 		"EIoErrorCode::CompressionError {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::PendingFork, 			PendingFork, 			"EIoErrorCode::PendingFork {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::PendingEncryptionKey, 	PendingEncryptionKey, 	"EIoErrorCode::PendingEncryptionKey {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::Disabled, 				Disabled, 				"EIoErrorCode::Disabled {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::NotInstalled, 			NotInstalled, 			"EIoErrorCode::NotInstalled {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::PendingHostGroup, 		PendingHostGroup, 		"EIoErrorCode::PendingHostGroup {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::Timeout, 				Timeout, 				"EIoErrorCode::Timeout {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::DeleteError, 			DeleteError, 			"EIoErrorCode::DeleteError {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::OutOfDiskSpace, 		OutOfDiskSpace, 		"EIoErrorCode::OutOfDiskSpace {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::FileSeekFailed, 		FileSeekFailed, 		"EIoErrorCode::FileSeekFailed {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::FileFlushFailed, 		FileFlushFailed, 		"EIoErrorCode::FileFlushFailed {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::FileMoveFailed, 		FileMoveFailed, 		"EIoErrorCode::FileMoveFailed {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::FileCloseFailed, 		FileCloseFailed, 		"EIoErrorCode::FileCloseFailed {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::FileCorrupt, 			FileCorrupt, 			"EIoErrorCode::FileCorrupt {ErrorMessage}", (FString, ErrorMessage));
UE_DECLARE_ERROR(UE_API, UE::IoStore, EIoErrorCode::HttpReadError, 			HttpReadError, 			"EIoErrorCode::HttpReadError {ErrorMessage}", (FString, ErrorMessage));

namespace UE::IoStore
{
	UE_API UE::UnifiedError::FError ConvertError(const FIoStatus& Status);
}

#undef UE_API
