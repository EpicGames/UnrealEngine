// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/RefCounting.h"

/**
 * Simple ref-counted interface that represents a resource that is held or required for the execution of some task item.
 */
class IExecutionResource : public IRefCountedObject
{
public:
	virtual ~IExecutionResource() = default;
};

/**
 * Per-thread stack-frame of execution resource currently held.
 * Can be retrieved and passed around to other task until not needed anymore.
 */
class FExecutionResourceContext
{
public:
	static CORE_API TRefCountPtr<IExecutionResource> Get();
};

/**
 * Used to push an execution resource on the stack-frame that can be retrieved by FExecutionResourceContext::Get().
 */
class FExecutionResourceContextScope
{
public:
	CORE_API FExecutionResourceContextScope(TRefCountPtr<IExecutionResource> InExecutionResource);
	CORE_API ~FExecutionResourceContextScope();
};
