// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnifiedError.h"

UE_DECLARE_ERROR_MODULE(CORE_API, UE::Core);

UE_DECLARE_ERROR(CORE_API, UE::Core, 1, ArgumentError,	   "Invalid argument '{ArgumentName}', reason '{ArgumentError}'", (FString, ArgumentName, TEXT("")), (FString, ArgumentError, TEXT("")));
UE_DECLARE_ERROR(CORE_API, UE::Core, 2, CancellationError, "The operation was cancelled");

namespace UE::Core
{

CORE_API bool IsCancellationError(const UE::UnifiedError::FError& Error);

} // namespace UE::Core

