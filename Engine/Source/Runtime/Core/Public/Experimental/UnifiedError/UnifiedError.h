// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/Utf8String.h"
#include "CoreTypes.h"
#include "HAL/PreprocessorHelpers.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Logging/StructuredLogFormat.h"
#include "Memory/MemoryFwd.h"
#include "Memory/MemoryView.h"
#include "Misc/TVariant.h"
#include "Misc/UEOps.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Templates/RefCounting.h"

// Temporarily allow some functions that we plan to remove in order to stage submissions across branches
#ifndef UE_UNIFIED_ERROR_DEPRECATED_FUNCTIONS
#define UE_UNIFIED_ERROR_DEPRECATED_FUNCTIONS 1
#endif

#include "UnifiedErrorDetails.h"

/**
 * This header provides types and macros for declaring and dealing with explicitly defined error conditions which can be 
 *  - stored uniformly regardless of origin
 *  - augmented with additional context as errors return from their direct failure point through layers of abstraction
 *	  to the original callers
 * 	- transformed into compact binary / json representations for analytics and pattern matching
 *  - transmitted across the network to a receiver which may or may not have the same error definitions
 *	  - if it does, the error can be reconstructed into the original C++ types (not yet implemented)
 *	  - otherwise, it can be left as compact binary and explored like a json document
 *
 * This header also contains guidance for migrating other error types to be convertible to FError. 
 *
 * ---------
 * Overview
 * ---------
 *
 * In a header, declare a module and as many errors as needed, then put matching DEFINEs in one .cpp:
 *
 *     UE_DECLARE_ERROR_MODULE(MYAPI, UE::Splines)
 *     UE_DECLARE_ERROR(MYAPI, UE::Splines, 1, ReticulationError,
 *         "Error reticulating spline {Name}", (FName, Name))
 *     // in .cpp: UE_DEFINE_ERROR_MODULE(UE::Splines), UE_DEFINE_ERROR(UE::Splines, ReticulationError)
 *
 * Use UE_DECLARE_ERROR_CONTEXT for a payload that can be attached to any error.
 *
 * The declaration is callable and returns FError.
 * PushErrorContext attaches additional payload and returns *this for chaining:
 *
 *     return UE::Splines::ReticulationError(Name)
 *                .PushErrorContext(UE::Splines::FMyContext{ .Reason = ... });
 *
 * Compare with `Err == UE::Splines::ReticulationError`. Retrieve a payload with
 * `Err.GetErrorContext<UE::Splines::ReticulationError>()`.
 *
 * For output, use CreateErrorMessage for localized FText, SerializeToCompactBinary or
 * SerializeToJsonForAnalytics for structured forms, or pass FError directly to UE_LOGFMT.
 *
 */

#define UE_API CORE_API

/**
 * Declares a namespace for errors associated with a logical code module.
 * The module name can be used in future macros to declare errors within this namespace.
 * UE_DEFINE_ERROR_MODULE must be used for the same module within a translation unit.
 * Should be used in the global namespace.
 * 
 * e.g. UE_DECLARE_ERROR_MODULE(UE_API, UE::Core)
 *
 * Declares a function DeclareApi int32 GetStaticModuleId() which can be used to retrieve the module
 * code for comparing against FError instances.
 *
 * @param DeclareApi API macro to export the declarations provided by the macro.
 * @param ModuleNamespace Namespace into which errors and related code will be inserted. 
 */
#define UE_DECLARE_ERROR_MODULE(DeclareApi, ModuleNamespace) UE_PRIVATE_DECLARE_ERROR_MODULE(DeclareApi, ModuleNamespace)

/**
 * Declares a particular error condition within an error module.
 * Requires a matching usage of UE_DEFINE_ERROR within a translation unit (subject to change)
 * Should be used in the global namespace.
 *
 * Additional fields may be added to the error using the variadic arguments to this macro. 
 * Each argument should be a parenthesized sequence of two or three arguments: field type, field name, and optionally default value.
 * e.g. (int32, Num, 0) or (int32, Num)
 * If no default value expression is provided, the field will be initialized with = {}, i.e. default constructed for class types or
 * value (zero) initialized for primitive types. 
 * 
 * The types used in this macro will be used as-is in storage for the error, so they should be safe to copy/move into long term storage.
 * i.e. types such as FStringView should not be used unless the pointed-to memory has a long enough lifetime, such as a string literal.
 * Types should be serializable to compact binary in one of three ways: a SerializeForError overload (preferred for
 * error-specific output), a SerializeForLog overload, or operator<<(FCbWriter&, const T&). The default SerializeForError
 * automatically routes through SerializeForLog and operator<<, so any existing log-serializable type works as a field.
 * 
 * Usage example: UE_DECLARE_ERROR(UE_API, UE::Splines, 1, ReticulationError, "Error reticulating spline {Name}", (FName, Name, {}))
 *
 * An error instance may then be created using the function generated by this macro:
 *  	UE::Splines::ReticulationError();
 *
 * In the case of errors with additional fields, the fields may be passed to to the factory method in order, or omitted to use default values.
 *  	UE::Splines::ReticulationError(Actor->GetFName());
 *
 * Fields may also be assigned using designated initializers, e.g.
 *  	UE::Splines::ReticulationError({ .Name = Actor->GetFName() });
 *
 * FError instances can be compared against the error declaration, i.e.
 *		FError Err = UE::Splines::ReticulationError();
 *		check(Err == UE::Splines::ReticulationError);
 *
 * The payload from creating the error may be retrieved by using the factory method or the error
 * payload type as a template parameter to FError::GetErrorContext
 * e.g. 
 * 		const UE::Splines::FReticulationError* Payload = Err.GetErrorContext<UE::Splines::FReticulationError>();
 * 		const UE::Splines::FReticulationError* Payload = Err.GetErrorContext<UE::Splines::ReticulationError>();
 *
 * Other types/fields/functions generated by this macro:
 * ModuleNamespace::ErrorName.ErrorCode -> int32 error code from the ErrorCode parameter to this macro
 * 
 * NOTE: If the signature of this macro is changed, UGatherTextFromSourceCommandlet needs to be updated.
 * 
 * @param DeclareApi  		API macro to export declarations from this macro for use in other modules. Use UE_EMPTY if this is not desired.
 * @param ModuleNamespace	Name of an error module previously declared with UE_DECLARE_ERROR_MODULE
 * @param ErrorCode			Unique code of this error within this module as a positive nonzero integer.
 *		May be an integer literal or an expression such as an enum member or constexpr variable.
 * @param ErrorName			Unique name of this error as an identifier.
 * @param FormatString  	A string literal including placeholders for inserting error parameters.
 *		A localization entry with the namespace ModuleName and the key ErrorName will be associated with this string.
 * @param __VA_ARGS__   	Parenthesized triples of type names, field names and default values to add additional data to the error. e.g. (uint32, HttpStatusCode, 0)
 */
#define UE_DECLARE_ERROR(DeclareApi, ModuleNamespace, ErrorCode, ErrorName, FormatString, ...) UE_PRIVATE_DECLARE_ERROR(DeclareApi, ModuleNamespace, ErrorCode, ErrorName, FormatString, ## __VA_ARGS__)

/** 
 * Declares a type which may be added to an error as additional context. 
 * This is very similar to an error declaration except it does not require an error code. 
 * Fields for the context can be provided through the variadic arguments as with UE_DECLARE_ERROR
 *
 * Declares a type ModuleNamespace::FContextName that can be used as a template argument (inferred) to 
 * FError::PushErrorContext.
 * e.g.
 *	Err.PushErrorContext<ModuleNamespace::FContextName>({ .Field1 = Value1 });
 * or
 *	Err.PushErrorContext(ModuleNamespace::FContextName{ .Field1 = Value1 });
 *
 * @param DeclareApi  		API macro to export declarations from this macro for use in other modules. Use UE_EMPTY if this is not desired.
 * @param ModuleNamespace	Name of an error module previously declared with UE_DECLARE_ERROR_MODULE
 * @param ContextName		Unique name of this context as an identifier.
 * @param FormatString  	A string literal including placeholders for inserting error parameters.
 *		A localization entry with the namespace ModuleName and the key ErrorName will be associated with this string.
 * @param __VA_ARGS__   	Parenthesized tuples of (FieldType, FieldName) or (FieldType, FieldName, DefaultValue) to add additional data to the context. e.g. (uint32, HttpStatusCode)
 */
#define UE_DECLARE_ERROR_CONTEXT(DeclareApi, ModuleNamespace, ContextName, FormatString, ...) UE_PRIVATE_DECLARE_ERROR_CONTEXT(DeclareApi, ModuleNamespace, ContextName, FormatString, ## __VA_ARGS__)

/**
 * Used in a translation unit to implement a previously declared module.
 * Should be used in the global namespace.
 * @see UE_DECLARE_ERROR_MODULE
 */
#define UE_DEFINE_ERROR_MODULE(ModuleNamespace) UE_PRIVATE_DEFINE_ERROR_MODULE(ModuleNamespace)

/**
 * Used in a translation unit to implement a previously declared error.
 * Should be used in the global namespace.
 * @see UE_DECLARE_ERROR
 */
#define UE_DEFINE_ERROR(ModuleNamespace, ErrorName) UE_PRIVATE_DEFINE_ERROR(ModuleNamespace, ErrorName)

/**
 * Used in a translation unit to implement a previously declared error context.
 * Should be used in the global namespace.
 * @see UE_DECLARE_ERROR_CONTEXT
 */
#define UE_DEFINE_ERROR_CONTEXT(ModuleNamespace, ContextName)  /* Nothing currently required, reserved for future use */



/**
 * MIGRATION GUIDE
 *
 * It is desirable for systems with their own existing error types to be compatible with FError so that
 * callers can deal with only FError for error handling and propagation.
 * e.g. if system HighLevelA calls functions in systems LowLevelB and LowLevelC, HighLevelA would like to
 * define its functions to return FError and to be able to pass through errors from LowLevelB and LowLevelC
 * through that function signature.
 *
 * Existing systems which return their own error types and cannot easily be changed to return FError can
 * provide compatibility with FError as follows:
 * - Declare a new set of desired errors with the UE_DECLARE_ERROR macros. These errors may map 1:1 or
 * 	 1:many to the existing error type, and should have fields to copy the details from the original error
 * 	 type into.
 * - These errors can use an existing error enum as a parameter to the UE_DECLARE_ERROR macro as long as
 *	 that enum can be cast to int32.
 * - If the existing error enum declares an entry for a value of 0, that cannot be used with UE_DECLARE_ERROR.
 *
 */





namespace UE::UnifiedError
{
	class FError;
	class FMandatoryErrorDetails;
	class IErrorDetails;
}

namespace UE::UnifiedError
{
	/* Flags controlling verbosity of various error outputs */
	enum class EDetailFilter : uint8
	{
		IncludeInSerialize = 1 << 0, // included when we serialize the details objects, almost every details should have this flag
		IncludeInAnalytics = 1 << 1, // what details objects to include when we are creating events for analytics
		IncludeInContextLogMessage = 1 << 2, // what details objects are included when the log message includes context objects on the string
		IncludeInLogMessage = 1 << 3, // which details objects are included when we log a message without context objects included
		// append here


		// standard values
		Default = IncludeInSerialize | IncludeInContextLogMessage,
		None = 0x00,
		All = 0xff
	};
	ENUM_CLASS_FLAGS(EDetailFilter);

	/** 
	 * The type of every error. Additional details are stored opaquely and can be serialized to text, logs, or compact binary for transmission over a network.
	 *
	 * FError instances are created with a function generated by error declarations. See UE_DECLARE_ERROR.
	 *
	 * Additional context can be added to an error as it propogates from the source of the error through the calling code with the method PushErrorContext.
	 * Errors can share the context added to them, so copying is cheap. However, each copy will accumulate future context independently.
	 * i.e. FError Err1 = ...;
	 * FError Err2 = Err1; <-- Err2 and Err1 have the same context
	 * Err1.PushErrorContext(...) <-- Err1 now has context which Err2 does not.
	 * 
	 * FErrors can be compared with == and !=, which will only compare the originating error, i.e. the module id and error code.
	 * Additional context will not change the result of the comparison.
	 */
	class FError
	{
	public:
		/** Primary constructor. Adds a ref count to the provided error details */
		FError(int32 InModuleId, int32 InErrorCode, const FMandatoryErrorDetails* InErrorDetails);
		/**
		 * Move constructor. Errors can be moved from, which removes all details so the error can
		 * no longer be correctly formatted as text etc.
		 */
		FError(FError&& InError);
		FError(const FError& InError) = default;
		~FError();

		/** Return the code for this error which is unique within its module */
		int32 GetErrorCode() const;
		/** Return the id for this module which is unique within this process  */
		int32 GetModuleId() const;

		/**
		 * Format this error as localized text. Includes details tagged with IncludeInLogMessage by default,
		 * and optionally those tagged with IncludeInContextLogMessage
		 * 
		 * @param bAppendContext Whether to append additional context added by intermediate error handlers.
		 */
		CORE_API FText CreateErrorMessage(bool bAppendContext = false) const;

#if UE_UNIFIED_ERROR_DEPRECATED_FUNCTIONS
		// Temporary old name of method for migration.
		FText GetErrorMessage(bool bAppendContext = false) const
		{
			return CreateErrorMessage(bAppendContext);
		}
#endif // UE_UNIFIED_ERROR_DEPRECATED_FUNCTIONS

		/**
		 * Add additional context to this error. This method is intended to allow errors to gather additional
		 * information as they bubble up the callstack from their origin, through different layers of abstraction.
		 *
		 * e.g. An error is created in low level filesystem code, specifiying the information available at the time.
		 * When the error returns to the caller, it adds additional context such as the asset it was attempting to load.
		 *
		 * Returns *this so callers can chain construction and context attachment in a single expression,
		 * e.g.
		 *	return MakeError(UE::Core::SomeError(FieldValue).PushErrorContext(MoveTemp(Ctx)));
		 */
		template<typename T> requires CErrorContext<RemoveCVRef_T<T>>
		FError& PushErrorContext(T&& InErrorStruct, const EDetailFilter& InDetailFilter = EDetailFilter::Default);

		/** 
		 * Returns context information previously added with PushErrorContext, or the original data the error was created with.
		 * The type parameter should be based on the declaration from UE_DECLARE_ERROR or UE_DECLARE_ERROR_CONTEXT.
		 *
		 * e.g. UE_DECLARE_ERROR(..., MyModule, MyError, ...)
		 * GetErrorContext<MyModule::FMyError>();
		 *
		 * Note that this only returns the original error data or data added with `PushErrorContext`. If you use a struct
		 * declared with UE_DECLARE_ERROR_CONTEXT as a field in `UE_DECLARE_ERROR`, that field will not be returned from
		 * this method.
		 */
		template<CErrorContext T>
		const T* GetErrorContext() const;

		/** 
		 * Helper for getting context information using the name of the error as declared rather than the name of the payload struct.
		 *
		 * e.g. UE_DECLARE_ERROR(..., MyModule, MyError, ...)
		 * GetErrorContext<MyModule::MyError>();
		 */
		template<CErrorDeclaration auto Declaration>
		const decltype(Declaration)::PayloadType* GetErrorContext() const;

		/** 
		 * Returns the name of the error as declared in UE_DECLARE_ERROR.
		 *
		 * e.g. for UE_DECLARE_ERROR(..., MyModule, MyError, ...)
		 * returns UTF8TEXTVIEW("MyError")
		 */
		CORE_API FUtf8StringView GetErrorCodeString() const;

		/** 
		 * Returns the name of the module in which this error was declared in UE_DECLARE_ERROR.
		 *
		 * e.g. for UE_DECLARE_ERROR(..., MyModule, MyError, ...)
		 * returns UTF8TEXTVIEW("MyModule")
		 */
		CORE_API FUtf8StringView GetModuleIdString() const;

		/** 
		 * Returns the name of the module in which this error was declared in UE_DECLARE_ERROR and its 
		 * name, joined with the namespace operator
		 *
		 * e.g. for UE_DECLARE_ERROR(..., MyModule, MyError, ...)
		 * returns UTF8TEXTVIEW("MyModule::MyError")
		 */
		CORE_API FUtf8StringView GetModuleIdAndErrorCodeString() const;

		/** 
		 * Writes this error to compact binary, filtering out details based on the passed in filter.
		 * Note that as detailed in @see FCbWriter, a compact binary document is a sequence of fields
		 * and not necessarily an object. This function writes the fields without a outer object wrapping them.
		 *
		 * Serializes the following fields in an unspecified order:
		 * $type - a string field with the value UE::UnifiedError::FError
		 * $format - a format string to print this error as unlocalized text
		 * $locformat, $locns, $lockey - format string and localization key & namespace to print this error as localized text
		 * ErrorCode - the numeric code assigned to the error in UE_DECLARE_ERROR
		 * ModuleId - a numeric code associated automatically with the module during compilation of the binary containing the error module declaration
		 * ErrorCodeString - The name of the error as declared in UE_DECLARE_ERROR
		 * ModuleIdString - The namespace of the error as declared in UE_DECLARE_ERROR / UE_DECLARE_ERROR_MODULE
		 * Details - an array containing the details of the initial error used to create this error, and any additional details added with PushErrorContext.
		 *  	Each details object is serialized according to its declaration with UE_DECLARE_ERROR / UE_DECLARE_ERROR_CONTEXT.
		 *		The details are filtered according to the EDetailFilter argument.	
		 */
		CORE_API void SerializeToCompactBinary(FCbWriter& InWriter, EDetailFilter InFilter) const;

		/** 
		 * Write this error to json for analytics. The format is the same as @see SerializeToCompactBinary
		 * except that the fields are wrapped in a top level object.
		 * The details serialized are filtered by EDetailFilter::IncludeInAnalytics.
		 */
		CORE_API FString SerializeToJsonForAnalytics() const;

		/** 
		 * Returns true if this error was correctly constructed and has not been moved from or invalidated.
		 */
		UE_FORCEINLINE_HINT bool IsValid() const;

		/** 
		 * Invalidates this error and discards all details as if it was moved from.
		 */
		CORE_API void Invalidate();

		/** 
		 * Comparison against ids from statically declared error. Supports comparison against error name in namespace
		 * e.g. FMyError Err = UE::Core::SomeError();
		 * check(Err == UE::Core::SomeError);
		 */
		bool Equals(int32 InModuleId, int32 InErrorCodeId) const;

		/** Comparison support via global operators for UEOpEquals. Only compares declared module and error code, no details. */
		bool UEOpEquals(const FError& Other) const;
	private:
		/** Error code from a UE_DECLARE_ERROR macro to identify the original error */
		int32 ErrorCode;
		/** Module id from UE_DECLARE_ERROR and UE_DECLARE_ERROR_MODULE macros */
		int32 ModuleId;
		/** 
		 * Linked list of error details. 
		 * If the error has not been moved from/invalidated, the first element in the list is the most recently
		 * added context and the last element is the original error payload
		 */
		TRefCountPtr<const IErrorDetails> ErrorDetails;

		/**
		 * Serialize all attached error details to the given FCbWriter
		 * Serializes an array field named "Details". See SerializeToCompactBinary for full format.
		 * Returns the mandatory error details if the error is valid (i.e. has not been moved from)
		 */
		const FMandatoryErrorDetails* SerializeDetails(FCbWriter& Writer, const EDetailFilter DetailFilter, bool bIncludeRoot = true) const;
		FString SerializeToJsonString(const EDetailFilter DetailFilter) const;

		/** 
		 * Returns the original details this error was created with, if the error has not been invalidated.
		 * If the error had no payload, this is an object shared by all instances of this error, otherwise 
		 * it is an object on the heap.
		 */
		const FMandatoryErrorDetails* GetMandatoryErrorDetails() const;

		/** Search the error details for an object with the matching declaration hash and return it if found */
		CORE_API TRefCountPtr<const IErrorDetails> GetErrorDetails(uint64 DeclarationHash) const;
	};

	UE_OPS_NAMESPACE_VISIBLE(FError);

	/** Integration for structured logging, serializing default details and wrapping the data in an object as structured logging expects. */
	CORE_API void SerializeForLog(FCbWriter& Writer, const class FError& Error);
}

#if UE_UNIFIED_ERROR_DEPRECATED_FUNCTIONS
inline FString LexToString(const UE::UnifiedError::FError& InError)
{
	return InError.CreateErrorMessage().ToString();
}
#endif // UE_UNIFIED_ERROR_DEPRECATED_FUNCTIONS




namespace UE::UnifiedError
{
	inline FError::FError(int32 InModuleId, int32 InErrorCode, const FMandatoryErrorDetails* InErrorDetails)
		: ErrorCode(InErrorCode)
		, ModuleId(InModuleId)
		, ErrorDetails(InErrorDetails, /* bAddRef */ true)
	{
	}

	inline FError::FError(FError&& InError)
	{
		ErrorCode = InError.ErrorCode;
		ModuleId = InError.ModuleId;
		if (InError.ErrorDetails)
		{
			ErrorDetails = MoveTemp(InError.ErrorDetails);
		}
		InError.Invalidate();
	}

	inline FError::~FError()
	{
		Invalidate();
	}

	inline int32 FError::GetErrorCode() const
	{
		return ErrorCode;
	}

	inline int32 FError::GetModuleId() const
	{
		return ModuleId;
	}

	inline bool FError::Equals(int32 InModuleId, int32 InErrorCodeId) const
	{
		return ModuleId == InModuleId && ErrorCode == InErrorCodeId;
	}

	template<typename T> requires CErrorContext<RemoveCVRef_T<T>>
	FError& FError::PushErrorContext(T&& InErrorStruct, const EDetailFilter& InDetailFilter /* = EDetailFilter::Default */)
	{
		check(ErrorDetails);
		ErrorDetails = new TErrorDetails<RemoveCVRef_T<T>>(ErrorDetails, Forward<T>(InErrorStruct), InDetailFilter);
		return *this;
	}

	template<CErrorContext T>
	const T* FError::GetErrorContext() const
	{
		TRefCountPtr<const IErrorDetails> Details = GetErrorDetails(T::DeclarationHash);
		if (Details)
		{
			FMemoryView Data = Details->GetErrorContext();
			if (!Data.IsEmpty())
			{ 
				checkf(Data.GetSize() == sizeof(T), TEXT("Size mismatch fetching error payload"));
				return reinterpret_cast<const T*>(Data.GetData());
			}
		}
		return nullptr;
	}

	template<CErrorDeclaration auto Declaration>
	const decltype(Declaration)::PayloadType* FError::GetErrorContext() const
	{
		return GetErrorContext<typename decltype(Declaration)::PayloadType>();
	}

	UE_FORCEINLINE_HINT bool FError::IsValid() const
	{
		return ErrorDetails != nullptr;
	}

	/** Comparison support via global operators for UEOpEquals */
	inline bool FError::UEOpEquals(const FError& Other) const
	{
		return ModuleId == Other.ModuleId && ErrorCode == Other.ErrorCode;
	}
} // namespace UnifiedError

#undef UE_API