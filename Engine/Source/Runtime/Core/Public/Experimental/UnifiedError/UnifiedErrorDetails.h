// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Concepts/ConvertibleTo.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Logging/StructuredLog.h"
#include "Templates/Function.h"
#include "Templates/RefCounting.h"

class FCbWriter;
class FText;

namespace UE
{
	/**
	 * Default SerializeForError for any type that already has a SerializeForLog overload.
	 * Lets error fields and payloads share their existing log serialization without
	 * duplicating overloads.
	 * To customize errors specifically, write an explicit SerializeForError overload.
	 * This allows errors to handle types for which it is undesirable to have SerializeForLog.
	 */
	template <typename ValueType UE_REQUIRES(TModels_V<CSerializableForLog, ValueType>)>
	inline void SerializeForError(FCbWriter& Writer, ValueType&& Value)
	{
		UE::CallSerializeForLog(Writer, (ValueType&&)Value);
	}

	/** Describes a type that can be serialized for use in an error payload or field. */
	struct CSerializableForError
	{
		template <typename ValueType>
		auto Requires(FCbWriter& Writer, ValueType&& Value) -> decltype(
			SerializeForError(Writer, (ValueType&&)Value)
		);
	};

	/** Wrapper to call SerializeForError with ADL from inside another SerializeForError overload. */
	template <typename ValueType UE_REQUIRES(TModels_V<CSerializableForError, ValueType>)>
	inline void CallSerializeForError(FCbWriter& Writer, ValueType&& Value)
	{
		SerializeForError(Writer, (ValueType&&)Value);
	}
}

// Error serializer for FText. Writes the current-locale string form so that FText can be used
// as a field type in UE_DECLARE_ERROR / UE_DECLARE_ERROR_CONTEXT. For localized format templates
// use UE::SerializeLogFormat in StructuredLogFormat.h instead.
CORE_API void SerializeForError(FCbWriter& Writer, const FText& Value);

// Generic error serializer for TArray. Routes through CallSerializeForError on each element so
// element types that customize via SerializeForError, or fall back through SerializeForLog or
// operator<<, all work uniformly.
template <typename ValueType, typename AllocatorType UE_REQUIRES(TModels_V<UE::CSerializableForError, ValueType>)>
inline void SerializeForError(FCbWriter& Writer, const TArray<ValueType, AllocatorType>& Array)
{
	Writer.BeginArray();
	for (const ValueType& Element : Array)
	{
		UE::CallSerializeForError(Writer, Element);
	}
	Writer.EndArray();
}

namespace UE::UnifiedError
{
	class FError;
	class FPrivate;
	enum class EDetailFilter : uint8;

	// Helper because C++20 <type_traits> is not available everywhere
	template<typename T>
	using RemoveCVRef_T = std::remove_cv_t<std::remove_reference_t<T>>;

	constexpr uint64 CompileTimeHashString(const UTF8CHAR* String)
	{
		// Based on djb2
		uint64 Result = 5381;
		while (char Ch = *String++)
		{
			Result = ((Result << 5) + Result) + Ch;
		}
		return Result;
	}

	/**
	 * This concept contains everything required for a struct which can be the payload of an
	 * error such as one defined with either UE_DECLARE_ERROR or UE_DECLARE_ERROR_CONTEXT
	 */
	template<typename T>
	concept CErrorPayload = requires(FCbWriter& Writer, const T& ConstRef)
	{
		// We need to be able to move or copy the data into the IErrorDetails
		requires std::is_move_constructible_v<T> || std::is_copy_constructible_v<T>;
		// We need to be able to identify the type from the IErrorDetails for reinterpet-casting
		{ T::DeclarationHash } -> UE::CConvertibleTo<uint64>;
		// We need to be able to write the type to compact binary for logging / sending over network.
		// SerializeForError has a default that routes to SerializeForLog (and from there to operator<<),
		// so any type that already supports either of those is automatically usable.
		{ SerializeForError(Writer, ConstRef) };
	};

	/**
	  * This concept contains everything required for a struct which can be added to an error
	  * with FError::PushErrorContext.
	  * These concepts are intentionally identical: it is legal to push the primary payload from 
	  * an error as a context onto another error.
	  */
	template<typename T>
	concept CErrorContext = CErrorPayload<T>;

	/**
	 * A type which can be used as a parameter to UE_DECLARE_ERROR or UE_DECLARE_ERROR_CONTEXT to
	 * define a field in the error payload.
	 */
	template<typename T>
	concept CErrorField = requires(FCbWriter& Writer, const T& ConstRef)
	{
		// We must be able to write the type into compact binary. The SerializeForError default
		// covers any type with SerializeForLog (and therefore any type with operator<< into FCbWriter).
		{ SerializeForError(Writer, ConstRef) };
	};

	/** 
	 * This concept represents a type generated by the UE_DECLARE_ERROR macro to hold static information about errors.
	 * It's separate from the type used as the error payload itself to keep these internal details out of that type
	 * which will be returned from FError::GetErrorContext
	 */
	template<typename T>
	concept CErrorDeclaration = requires (const T& t, T::PayloadType&& p)
	{
		// Stored payload type
		typename T::PayloadType;
		// Which must conform to CErrorPayload
		requires CErrorPayload<typename T::PayloadType>;
		// Whether or not this is an empty payload, i.e. there is no per-instance data for this error type
		{ T::bPayloadHasFields } -> UE::CConvertibleTo<bool>;
		// Hash of declaration name to look up error payload for runtime casting
		{ T::DeclarationHash } -> UE::CConvertibleTo<uint64>;
		// Declared error code
		{ T::ErrorCode } -> UE::CConvertibleTo<int32>;
		// Declared error code cannot be 0
		requires T::ErrorCode != 0;
		// Declared module name
		{ T::StaticModuleName } -> UE::CConvertibleTo<FUtf8StringView>;
		// DeclaredErrorName
		{ T::StaticErrorName } -> UE::CConvertibleTo<FUtf8StringView>;
		// Declared full name with module and error name
		{ T::StaticFullErrorName } -> UE::CConvertibleTo<FUtf8StringView>;
		// Declare format string to print error & payload
		{ T::FormatString } -> UE::CConvertibleTo<FText>;
		// Factory method taking payload type to return error
		{ t(MoveTemp(p)) } -> UE::CConvertibleTo<UE::UnifiedError::FError>;
	};

}

#define UE_PRIVATE_DECLARE_ERROR_MODULE(DeclareApi, ModuleNamespace) \
	namespace ModuleNamespace { \
		constexpr FUtf8StringView StaticModuleName = UTF8TEXTVIEW(#ModuleNamespace); \
		DeclareApi int32 GetStaticModuleId(); \
	}

#define UE_PRIVATE_DEFINE_ERROR_MODULE(ModuleNamespace) \
	namespace ModuleNamespace { \
		const int32 StaticModuleId = UE::UnifiedError::ErrorRegistry::RegisterModule(UTF8TEXTVIEW(#ModuleNamespace)); \
		int32 GetStaticModuleId() { return StaticModuleId; } \
	}


#define UE_PRIVATE_DECLARE_ERROR(DeclareApi, ModuleNamespace, InErrorCode, ErrorName, FormatStringLiteral, ...) \
	namespace ModuleNamespace \
	{ \
	/* Payload struct */ \
	struct F##ErrorName \
	{ \
		static inline constexpr uint64 DeclarationHash = UE::UnifiedError::CompileTimeHashString(UTF8TEXT(#ModuleNamespace) UTF8TEXT(#ErrorName)); \
		UE_FOR_EACH(UE_PRIVATE_DECLARE_ERROR_FIELD, ## __VA_ARGS__); \
	}; \
	void SerializeForError(FCbWriter& Writer, const ModuleNamespace::F##ErrorName& Context); \
	/* Factory object/comparison object for error */ \
	struct FErrorDeclaration_##ErrorName {\
		using PayloadType = F##ErrorName; \
		static inline constexpr bool bPayloadHasFields = (UE_VA_ARG_COUNT(__VA_ARGS__) != 0); \
		static inline constexpr uint64 DeclarationHash = F##ErrorName::DeclarationHash; \
		static inline constexpr int32 ErrorCode = static_cast<int32>(InErrorCode); \
		static inline constexpr FUtf8StringView StaticModuleName = UTF8TEXTVIEW(#ModuleNamespace); \
		static inline constexpr FUtf8StringView StaticErrorName = UTF8TEXTVIEW(#ErrorName); \
		static inline constexpr FUtf8StringView StaticFullErrorName = UTF8TEXTVIEW(#ModuleNamespace "::" #ErrorName); \
		static inline constexpr FUtf8StringView TypeNameString = UTF8TEXTVIEW(#ModuleNamespace "::F" #ErrorName ); \
		static DeclareApi const FText FormatString; \
		static_assert(ErrorCode != 0, "Declared error code for " #ModuleNamespace "::" #ErrorName " cannot be zero"); \
		static DeclareApi UE::UnifiedError::FError GetStaticError(); \
		static inline FText CreateFormatString() \
		{ \
			return NSLOCTEXT(#ModuleNamespace, #ErrorName, FormatStringLiteral); \
		} \
		inline UE::UnifiedError::FError operator()(\
			UE_FOR_EACH_WITH_SEPARATOR(UE_PRIVATE_ERROR_FIELD_PARAMETER, UE_COMMA, ## __VA_ARGS__) \
		) const \
		{ \
			return (*this)(F##ErrorName { UE_FOR_EACH_WITH_SEPARATOR(UE_PRIVATE_ASSIGN_ERROR_FIELD, UE_COMMA, ## __VA_ARGS__) }); \
		} \
		inline UE::UnifiedError::FError operator()(F##ErrorName&& InContext) const { \
			if constexpr (bPayloadHasFields) \
			{ \
				return UE::UnifiedError::FError(GetStaticModuleId(), \
					ErrorCode, \
					new UE::UnifiedError::TMandatoryErrorDetails<FErrorDeclaration_##ErrorName>(MoveTemp(InContext))); \
			} \
			else \
			{ \
				return GetStaticError(); \
			} \
		} \
		UE_REWRITE bool UEOpEquals(const UE::UnifiedError::FError& Other) const \
		{ \
			return Other.Equals(GetStaticModuleId(), ErrorCode); \
		} \
		friend class UE::UnifiedError::FPrivate; \
	}; \
	static_assert(UE::UnifiedError::CErrorDeclaration<ModuleNamespace::FErrorDeclaration_##ErrorName>); \
	/* Singleton instance of factory object */ \
	constexpr FErrorDeclaration_##ErrorName ErrorName; \
	inline void SerializeForError(FCbWriter& Writer, const ModuleNamespace::F##ErrorName& Context) \
	{ \
		using ErrorType = F##ErrorName; \
		using DeclarationType = FErrorDeclaration_##ErrorName; \
		UE_FOR_EACH(UE_PRIVATE_CHECK_ERROR_FIELD, ## __VA_ARGS__); \
		Writer.BeginObject(); \
		Writer.AddString(UTF8TEXTVIEW("$type"), DeclarationType::TypeNameString); \
		UE::SerializeLogFormat(Writer, DeclarationType::FormatString); \
		UE_FOR_EACH(UE_PRIVATE_SERIALIZE_ERROR_FIELD, ## __VA_ARGS__); \
		Writer.EndObject(); \
	} \
	} \
	static_assert(UE::UnifiedError::CErrorPayload<ModuleNamespace::F##ErrorName>);

#define UE_PRIVATE_DEFINE_ERROR(ModuleNamespace, ErrorName) \
	namespace ModuleNamespace { \
		const FText FErrorDeclaration_##ErrorName::FormatString = CreateFormatString(); \
		UE::UnifiedError::FError FErrorDeclaration_##ErrorName::GetStaticError() \
		{ \
			return UE::UnifiedError::FError(GetStaticModuleId(), ErrorCode, UE::UnifiedError::Private::GetSharedStaticErrorDetails<FErrorDeclaration_##ErrorName>()); \
		} \
	}

#define UE_PRIVATE_DECLARE_ERROR_CONTEXT(DeclareApi, ModuleNamespace, ContextName, FormatString, ...) \
	namespace ModuleNamespace \
	{ \
		struct F##ContextName \
		{ \
			static inline constexpr uint64 DeclarationHash = UE::UnifiedError::CompileTimeHashString(UTF8TEXT(#ModuleNamespace) UTF8TEXT(#ContextName)); \
			UE_FOR_EACH(UE_PRIVATE_DECLARE_ERROR_FIELD, ## __VA_ARGS__); \
		}; \
		struct FContextDeclaration_##ContextName \
		{ \
			using PayloadType = F##ContextName; \
			static inline constexpr uint64 DeclarationHash = F##ContextName::DeclarationHash; \
			static inline constexpr FUtf8StringView TypeNameString = UTF8TEXTVIEW(#ModuleNamespace "::F" #ContextName); \
		}; \
	} \
	inline void SerializeForError(FCbWriter& Writer, const ModuleNamespace::F##ContextName& Context) \
	{ \
		using ContextType = ModuleNamespace::F##ContextName; \
		using DeclarationType = ModuleNamespace::FContextDeclaration_##ContextName; \
		Writer.BeginObject(); \
		Writer.AddString(UTF8TEXTVIEW("$type"), DeclarationType::TypeNameString); \
		UE::SerializeLogFormat(Writer, NSLOCTEXT(#ModuleNamespace, #ContextName, FormatString)); \
		UE_FOR_EACH(UE_PRIVATE_SERIALIZE_ERROR_FIELD, ## __VA_ARGS__); \
		Writer.EndObject(); \
	} \
	static_assert(UE::UnifiedError::CErrorContext<ModuleNamespace::F##ContextName>);

/** Check that each field is a valid type */
#define UE_PRIVATE_CHECK_ERROR_FIELD(...) UE_APPEND_VA_ARG_COUNT(UE_PRIVATE_CHECK_ERROR_FIELD, __VA_ARGS__)(__VA_ARGS__)
#define UE_PRIVATE_CHECK_ERROR_FIELD2(FieldType, FieldName) static_assert(UE::UnifiedError::CErrorField<UE_REMOVE_OPTIONAL_PARENS(FieldType)>, \
	"Field " UE_STRINGIZE(FieldName) " of type " UE_STRINGIZE(FieldType) " is not a valid type for an error field. Field types must satisfy the concept UE::UnifiedError::CErrorField");
#define UE_PRIVATE_CHECK_ERROR_FIELD3(FieldType, FieldName, DefaultValue) static_assert(UE::UnifiedError::CErrorField<UE_REMOVE_OPTIONAL_PARENS(FieldType)>, \
	"Field " UE_STRINGIZE(FieldName) " of type " UE_STRINGIZE(FieldType) " is not a valid type for an error field. Field types must satisfy the concept UE::UnifiedError::CErrorField");
/** Macro to declare each error field inside a struct e.g. FName Name = {}; */
#define UE_PRIVATE_DECLARE_ERROR_FIELD(...)  UE_APPEND_VA_ARG_COUNT(UE_PRIVATE_DECLARE_ERROR_FIELD, __VA_ARGS__)(__VA_ARGS__)
#define UE_PRIVATE_DECLARE_ERROR_FIELD3(FieldType, FieldName, DefaultValue)  UE_REMOVE_OPTIONAL_PARENS(FieldType) FieldName = UE_REMOVE_OPTIONAL_PARENS(DefaultValue);
#define UE_PRIVATE_DECLARE_ERROR_FIELD2(FieldType, FieldName)  UE_PRIVATE_DECLARE_ERROR_FIELD3(FieldType, FieldName, {}) 
/** Macro to declare serialization to FCbWriter for each error field */
#define UE_PRIVATE_SERIALIZE_ERROR_FIELD(...)  UE_APPEND_VA_ARG_COUNT(UE_PRIVATE_SERIALIZE_ERROR_FIELD, __VA_ARGS__)(__VA_ARGS__)
#define UE_PRIVATE_SERIALIZE_ERROR_FIELD3(FieldType, FieldName, DefaultValue) UE_PRIVATE_SERIALIZE_ERROR_FIELD2(FieldType, FieldName)
#define UE_PRIVATE_SERIALIZE_ERROR_FIELD2(FieldType, FieldName) Writer.SetName(UTF8TEXTVIEW(#FieldName)); UE::CallSerializeForError(Writer, Context.FieldName);
/** Macro to declare parameters in function signature for an error, e.g. FName Name = {} */
#define UE_PRIVATE_ERROR_FIELD_PARAMETER(...)  UE_APPEND_VA_ARG_COUNT(UE_PRIVATE_ERROR_FIELD_PARAMETER, __VA_ARGS__)(__VA_ARGS__)
#define UE_PRIVATE_ERROR_FIELD_PARAMETER3(FieldType, FieldName, DefaultValue) UE_REMOVE_OPTIONAL_PARENS(FieldType) FieldName = UE_REMOVE_OPTIONAL_PARENS(DefaultValue)
#define UE_PRIVATE_ERROR_FIELD_PARAMETER2(FieldType, FieldName) UE_PRIVATE_ERROR_FIELD_PARAMETER3(FieldType, FieldName, {})
/** Macro to assign error fields from parameters of matching names for an error, e.g. .Name = MoveTemp(Name) */
#define UE_PRIVATE_ASSIGN_ERROR_FIELD(FieldType, FieldName, ...) .FieldName = MoveTemp(FieldName)

namespace UE::UnifiedError
{
	namespace ErrorRegistry
	{
		CORE_API uint32 RegisterModule(FUtf8StringView ModuleName);

		CORE_API int32 RegisterErrorCode(FUtf8StringView ErrorName, int32 ModuleId, int32 ErrorCode);
	}

	class IErrorDetails : public IRefCountedObject
	{
	public:
		virtual ~IErrorDetails() = default;

		/**
		 * Exposes inner error details to FError, if any. This should always return null for mandatory error details, as they should be the last in the list.
		 */
		virtual TRefCountPtr<const IErrorDetails> GetInnerErrorDetails() const = 0;

		/**
		 * GetErrorDetialsTypeId; Simple type information for error details, generated using hash of details name 
		 * @see #define FERROR_DETAILS
		 */
		virtual uint64 GetErrorDetailsTypeId() const = 0;

		/**
		 * Serialize these details to compact binary for logging, analytics, localization etc. 
		 */
		virtual void SerializeToCompactBinary(FCbWriter& Writer) const = 0;

		/** Returns whether these details should be included in some kind of serialization/conversion, e.g. logging, analytics */
		virtual bool ShouldInclude(const EDetailFilter DetailFilter) const = 0;
	
		// Returns the stored context for this error if any. Must be used with a type checking facility (see DeclarationHash).
		virtual FMemoryView GetErrorContext() const = 0;
	};

	class FMandatoryErrorDetails : public IErrorDetails
	{
	public:
		virtual bool ShouldInclude(const EDetailFilter DetailFilter) const final
		{
			return true;
		}
		virtual TRefCountPtr<const IErrorDetails> GetInnerErrorDetails() const final
		{
			return nullptr;
		}
		virtual FUtf8StringView GetErrorCodeString(const FError& Error) const = 0;
		virtual FUtf8StringView GetModuleIdString(const FError& Error) const = 0;
		virtual FUtf8StringView GetModuleIdAndErrorCodeString(const FError& Error) const = 0;
	};

	// When there are fields in the error, we allocate the mandatory details on the heap with the payload
	// And therefore need refcounting.
	template<CErrorDeclaration ErrorDeclarationType, bool bHasFields = ErrorDeclarationType::bPayloadHasFields>
	class TMandatoryErrorDetailsRefCounting : public FMandatoryErrorDetails, public FRefCountedObject
	{
	public:
		virtual void AddRef() const final override 
		{
			FRefCountedObject::AddRef();
		}
		virtual FReturnedRefCountValue Release() const final override 
		{
			return FRefCountedObject::Release();
		}
		virtual FReturnedRefCountValue GetRefCount() const final override 
		{
			return FRefCountedObject::GetRefCount();
		}
	};

	// When there are no fields on the error, we don't need AddRef/Release to do anything as all errors
	// share a static instance
	template<CErrorDeclaration ErrorDeclarationType>
	class TMandatoryErrorDetailsRefCounting<ErrorDeclarationType, false> : public FMandatoryErrorDetails
	{
	public:
		virtual void AddRef() const final 
		{
		}
		virtual FReturnedRefCountValue Release() const final 
		{
			// Return hardcoded immutable refcount so ref counted pointers to this type never try to free this static singleton
			return FReturnedRefCountValue{ 10 }; 
		}
		virtual FReturnedRefCountValue GetRefCount() const final 
		{
			return FReturnedRefCountValue{ 10 };
		}
	};

	// ErrorStruct is the public type containing fields declared with the error, i.e. ModuleNamespace::FErrorName
	// ErrorDeclarationType is the private type containing information generated by the macro
	// These could be merged.
	template<CErrorDeclaration ErrorDeclarationType>
	class TMandatoryErrorDetails : public TMandatoryErrorDetailsRefCounting<ErrorDeclarationType>
	{
		using PayloadType = ErrorDeclarationType::PayloadType;
		// If this type is empty, there are no instance fields and all instances of this error can share the same pointer to this struct
		UE_NO_UNIQUE_ADDRESS PayloadType Payload;
	public:
		TMandatoryErrorDetails(PayloadType&& InPayload)
		: Payload(MoveTemp(InPayload))
		{
		}

		virtual FUtf8StringView GetErrorCodeString(const FError& Error) const final
		{
			return ErrorDeclarationType::StaticErrorName;
		}
		virtual FUtf8StringView GetModuleIdString(const FError& Error) const final
		{
			return ErrorDeclarationType::StaticModuleName;
		}
		virtual FUtf8StringView GetModuleIdAndErrorCodeString(const FError& Error) const final
		{
			return ErrorDeclarationType::StaticFullErrorName;
		}
		virtual uint64 GetErrorDetailsTypeId() const final
		{
			return ErrorDeclarationType::DeclarationHash;
		}
		virtual void SerializeToCompactBinary(FCbWriter& Writer) const final
		{
			UE::CallSerializeForError(Writer, Payload);
		}
		virtual FMemoryView GetErrorContext() const final
		{
			if constexpr (ErrorDeclarationType::bPayloadHasFields)
			{
				return MakeMemoryView(&Payload, sizeof(Payload));
			}
			else 
			{
				return FMemoryView{};
			}
		}
	};

	namespace Private
	{
		// For errors declared with UE_DECLARE_ERROR that have no payload fields, there is a single
		// shared TMandatoryErrorDetails instance used by every FError of that type — it has no
		// per-instance data to store. This helper returns that instance (constructed on first call).
		//
		// For errors with payload fields, this helper is never called at runtime (FError operator()
		// takes the fields-branch in an if-constexpr) and returns nullptr. The default-construction
		// of PayloadType is guarded by the same if-constexpr so payload types are not required to
		// be default-constructible.
		template<CErrorDeclaration ErrorDeclarationType>
		const FMandatoryErrorDetails* GetSharedStaticErrorDetails()
		{
			if constexpr (!ErrorDeclarationType::bPayloadHasFields)
			{
				static TMandatoryErrorDetails<ErrorDeclarationType> Details{typename ErrorDeclarationType::PayloadType{}};
				return &Details;
			}
			else
			{
				return nullptr;
			}
		}
	}

	class FErrorDetailsRegistry
	{
	private:
		struct FRegisteredDetails
		{
			FUtf8StringView Name;
			TFunction<IErrorDetails*()> CreateFunction;
		};
		TMap<uint64, FRegisteredDetails> CreateFunctions;

		FErrorDetailsRegistry() { }
	public:
		static FErrorDetailsRegistry& Get()
		{
			static FErrorDetailsRegistry Registry;
			return Registry;
		}

		CORE_API void RegisterDetails(uint64 DeclarationHash, FUtf8StringView ErrorDetailsName, TFunction<IErrorDetails*()> CreationFunction);
	};

	template<CErrorContext T>
	class TErrorDetails : public IErrorDetails, public FRefCountedObject
	{
	public:
		template <typename U> requires std::is_same_v<RemoveCVRef_T<U>, T>
		TErrorDetails(TRefCountPtr<const IErrorDetails> InNext, U&& InPayload, const EDetailFilter InDetailFilterMask)
			: Payload(Forward<U>(InPayload))
			, DetailFilterMask(InDetailFilterMask)
			, InnerErrorDetails(MoveTemp(InNext))
		{
		}

		virtual TRefCountPtr<const IErrorDetails> GetInnerErrorDetails() const 
		{
			return InnerErrorDetails;
		}

		virtual uint64 GetErrorDetailsTypeId() const
		{
			return T::DeclarationHash;
		}

		virtual bool ShouldInclude(const EDetailFilter InDetailFilter) const override
		{
			return EnumHasAnyFlags(InDetailFilter, DetailFilterMask);
		}

		virtual void AddRef() const final override 
		{
			FRefCountedObject::AddRef(); 
		}
		virtual FReturnedRefCountValue Release() const final override 
		{
			return FRefCountedObject::Release(); 
		}
		virtual FReturnedRefCountValue GetRefCount() const final override 
		{
			return FRefCountedObject::GetRefCount(); 
		}

	private:
		T Payload;
		EDetailFilter DetailFilterMask;
		TRefCountPtr<const IErrorDetails> InnerErrorDetails;
		
	public:
		FMemoryView GetErrorContext() const
		{
			return FMemoryView(&Payload, sizeof(Payload));
		}

		virtual void SerializeToCompactBinary(FCbWriter& Writer) const override
		{
			UE::CallSerializeForError(Writer, Payload);
		}
	};
} // namespace UnifiedError
