// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreTypes.h"

/**
 * All modular feature interfaces derive from this base interface
 */
class IModularFeature
{

public:


private:

};

