// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "UObject/NameTypes.h"
#include "Delegates/Delegate.h"
#include "IModularFeature.h"

/**
 * Public interface to all known modular features
 */
class IModularFeatures
{

public:

	/**
	 * Static: Access singleton instance
	 *
	 * @return	Reference to the singleton object
	 */
	static CORE_API IModularFeatures& Get();

	/** Virtual destructor, to make polymorphism happy. */
	virtual ~IModularFeatures() = default;


	/**
	 * Checks to see if the specified feature is available
	 *
	 * @param	Type	The type of feature we're looking for
	 *
	 * @return	True if the feature is available right now and it is safe to call GetModularFeature()
	 */
	inline bool IsModularFeatureAvailable( const FName Type )
	{
		return GetModularFeatureImplementationCount( Type ) > 0;
	}


	/**
	 * Queries for a specific feature.  If multiple implementations of the same feature were registered, this will return the first.  Will assert or crash 
	 * if the specified feature is not available!  You should call IsModularFeatureAvailable() first!
	 *
	 * @param	Type	The type of feature we're looking for
	 *
	 * @return	The feature.
	 */
	template< typename TModularFeature >
	inline TModularFeature& GetModularFeature( const FName Type )
	{
		return static_cast<TModularFeature&>( *GetModularFeatureImplementation( Type, 0 ) );
	}


	/**
	 * Queries for one or more implementations of a single feature.  If no feature of this type is registered, will return an empty array.
	 *
	 * @param	Type	The type of feature we're looking for
	 *
	 * @return	List of available implementations of this feature.
	 */
	template< typename TModularFeature >
	inline TArray< TModularFeature* > GetModularFeatureImplementations( const FName Type )
	{
		TArray< TModularFeature* > FeatureImplementations;
		const int32 NumImplementations = GetModularFeatureImplementationCount( Type );
		for( int32 CurImplementation = 0; CurImplementation < NumImplementations; ++CurImplementation )
		{
			FeatureImplementations.Add( static_cast<TModularFeature*>( GetModularFeatureImplementation( Type, CurImplementation ) ) );
		}
		return FeatureImplementations;
	}


public:

	/** Lock modular features for access from another thread */
	virtual void LockModularFeatureList() = 0;

	/** Unlock modular features when finished accessing from another thread */
	virtual void UnlockModularFeatureList() = 0;

	struct FScopedLockModularFeatureList
	{
		inline FScopedLockModularFeatureList() { IModularFeatures::Get().LockModularFeatureList(); }
		inline ~FScopedLockModularFeatureList() { IModularFeatures::Get().UnlockModularFeatureList(); }

		// Non-copyable :
		FScopedLockModularFeatureList(const FScopedLockModularFeatureList&) = delete;
		FScopedLockModularFeatureList(FScopedLockModularFeatureList&&) = delete;
		FScopedLockModularFeatureList& operator = (const FScopedLockModularFeatureList&) = delete;
		FScopedLockModularFeatureList& operator = (FScopedLockModularFeatureList&&) = delete;
	};

	/**
	 * Returns the number of registered implementations of the specified feature type.
	 *
	 * @param	Type	The type of feature we're looking for
	 *
	 * @return	Number of implementations of this feature.
	 */
	virtual int32 GetModularFeatureImplementationCount( const FName Type ) = 0;

	/**
	 * Queries for a specific modular feature.  Returns NULL if the feature is not available.  Does not assert.
	 * Usually you should just call GetModularFeature instead, after calling IsModularFeatureAvailable().
	 *
	 * @param	Type	The type of feature we're looking for
	 * @param	Index	The index of the implementation (there may be multiple implementations of the same feature registered.)
	 *
	 * @return	Pointer to the feature interface, or NULL if it's not available right now.
	 */
	virtual class IModularFeature* GetModularFeatureImplementation( const FName Type, const int32 Index ) = 0;

	/**
	 * Registers a feature.  Usually called by plugins to augment or replace existing modular features.
	 *
	 * @param	Type			The type of feature we're registering
	 * @param	ModularFeature	Interface to the modular feature object.  We do not assume ownership of this object.  It's up to you to keep it allocated until it is unregistered later on.
	 */
	virtual void RegisterModularFeature( const FName Type, class IModularFeature* ModularFeature ) = 0;


	/**
	 * Unregisters a feature that was registered earlier on.  After unregistering a feature, other systems will no longer be able to gain access to it through this interface.
	 *
	 * @param	Type			The type of feature we're unregistering
	 * @param	ModularFeature	Interface to the modular feature object
	 */
	virtual void UnregisterModularFeature( const FName Type, class IModularFeature* ModularFeature ) = 0;

	/** 
	 * Event used to inform clients that a modular feature has been registered.
	 *
	 * @param	Type	The name of the modular feature type being registered.
	 */
	DECLARE_EVENT_TwoParams(IModularFeatures, FOnModularFeatureRegistered, const FName& /** Type */, class IModularFeature* /*ModularFeature*/);
	virtual FOnModularFeatureRegistered& OnModularFeatureRegistered() = 0;

	/** 
	 * Event used to inform clients that a modular feature has been unregistered.
	 *
	 * @param	Type	The name of the modular feature type being unregistered.
	 */
	DECLARE_EVENT_TwoParams(IModularFeatures, FOnModularFeatureUnregistered, const FName& /** Type */, class IModularFeature* /*ModularFeature*/);
	virtual FOnModularFeatureUnregistered& OnModularFeatureUnregistered() = 0;
};

