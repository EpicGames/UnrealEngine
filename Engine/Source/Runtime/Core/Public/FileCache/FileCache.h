// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/Platform.h"
#include "HAL/PlatformFile.h"
#include "Misc/MemoryReadStream.h"

class IAsyncReadFileHandle;

//
struct FFileCachePreloadEntry
{
	FFileCachePreloadEntry() : Offset(0), Size(0) {}
	FFileCachePreloadEntry(int64 InOffset, int64 InSize) : Offset(InOffset), Size(InSize) {}

	int64 Offset;
	int64 Size;
};

/**
 * All methods may be safely called from multiple threads simultaneously, unless otherwise noted
 *
 * Also note, if you create several IFileCacheHandle's to the same file on separate threads these will be considered
 * as individual separate files from the cache point of view and thus each will have their own cache data allocated.
 */
class IFileCacheHandle
{
public:
	CORE_API static void EvictAll();

	/**
	 * Create a IFileCacheHandle from a filename.
	 * @param	InFileName			A path to a file.
	 * @return	A IFileCacheHandle that can be used to make read requests. This will be a nullptr if the target file can not be accessed
	 *			for any given reason.
	 */
	CORE_API static IFileCacheHandle* CreateFileCacheHandle(const TCHAR* InFileName, int64 InBaseOffset = 0);

	/**
	 * Create a IFileCacheHandle from a IAsyncReadFileHandle.
	 * @param	FileHandle			A valid IAsyncReadFileHandle that has already been created elsewhere.
	 * @return	A IFileCacheHandle that can be used to make read requests. This will be a nullptr if the FileHandle was not valid.
	 */
	CORE_API static IFileCacheHandle* CreateFileCacheHandle(IAsyncReadFileHandle* FileHandle, int64 InBaseOffset = 0);

	virtual ~IFileCacheHandle() = default;

	/** Return size of underlying file cache in bytes. */
	CORE_API static int64 GetFileCacheSize();

	/**
	 * Read a byte range form the file. This can be a high-throughput operation and done lots of times for small reads.
	 * The system will handle this efficiently.
	 * @param	OutCompletionEvents			Must wait until these events are complete before returned data is valid
	 * @return	Memory stream that contains the requested range.  May return nullptr in rare cases if the request could not be serviced
	 *			Data read from this stream will not be valid until all events returned in OutCompletionEvents are complete
	 */
	virtual IMemoryReadStreamRef ReadData(FGraphEventArray& OutCompletionEvents, int64 Offset, int64 BytesToRead, EAsyncIOPriorityAndFlags Priority) = 0;

	virtual FGraphEventRef PreloadData(const FFileCachePreloadEntry* PreloadEntries, int32 NumEntries, EAsyncIOPriorityAndFlags Priority) = 0;

	/**
	 * Wait until all outstanding read requests complete.
	 */
	virtual void WaitAll() = 0;
};

#if !UE_BUILD_SHIPPING
//
// Called by the IoStore system to inform the FileCache of used compression block sizes,
// for usage warning purposes. (FFileIoStoreReader::ReadContainerHeader)
//
void CORE_API FileCache_PostIoStoreCompressionBlockSize(uint32 InCompressionBlockSize, FString const& InContainerFilePath);
#endif // !UE_BUILD_SHIPPING
