/*
Copyright 2019 PureDev Software Limited

Permission to use, copy, modify, and/or distribute this software for any
purpose with or without fee is hereby granted, provided that the above
copyright notice and this permission notice appear in all copies.

THE SOFTWARE IS PROVIDED "AS IS" AND THE AUTHOR DISCLAIMS ALL WARRANTIES
WITH REGARD TO THIS SOFTWARE INCLUDING ALL IMPLIED WARRANTIES OF
MERCHANTABILITY AND FITNESS. IN NO EVENT SHALL THE AUTHOR BE LIABLE FOR
ANY SPECIAL, DIRECT, INDIRECT, OR CONSEQUENTIAL DAMAGES OR ANY DAMAGES
WHATSOEVER RESULTING FROM LOSS OF USE, DATA OR PROFITS, WHETHER IN AN
ACTION OF CONTRACT, NEGLIGENCE OR OTHER TORTIOUS ACTION, ARISING OUT OF
OR IN CONNECTION WITH THE USE OR PERFORMANCE OF THIS SOFTWARE.
*/

//------------------------------------------------------------------------
//
// FramePro.h
//
//------------------------------------------------------------------------
/*
	FramePro
	Version:	1.10.10.0
*/
//------------------------------------------------------------------------
#ifndef FRAMEPRO_H_INCLUDED
#define FRAMEPRO_H_INCLUDED

//------------------------------------------------------------------------
// *** enable/disable FramePro here
//@EPIC BEGIN: Enable through minimal config file, and enable for engine / game, not external tools
#include "FramePro/FrameProConfig.h"
//@EPIC END

//------------------------------------------------------------------------
// Scroll down for detailed documentation on all of these macros.
#if FRAMEPRO_ENABLED

	// session macros
	#define FRAMEPRO_FRAME_START()															FramePro::FrameStart()
	#define FRAMEPRO_SHUTDOWN()																FramePro::Shutdown()
	#define FRAMEPRO_SET_PORT(port)															FramePro::SetPort(port)
	#define FRAMEPRO_SESSION_INFO(name, value)												FramePro::SendSessionInfo(name, value)
	#define FRAMEPRO_SET_ALLOCATOR(p_allocator)												FramePro::SetAllocator(p_allocator)
	#define FRAMEPRO_SET_THREAD_NAME(name)													FramePro::SetThreadName(name)
	#define FRAMEPRO_SET_THREAD_ID_NAME(thread_id, name)									FramePro::SetThreadName(thread_id, name)
	#define FRAMEPRO_THREAD_ORDER(thread_name)												FramePro::SetThreadOrder(FramePro::RegisterString(thread_name))
	#define FRAMEPRO_REGISTER_STRING(str)													FramePro::RegisterString(str)
	#define FRAMEPRO_START_RECORDING(filename, context_switches, callstacks, max_file_size)	FramePro::StartRecording(filename, context_switches, callstacks, max_file_size)
	#define FRAMEPRO_STOP_RECORDING()														FramePro::StopRecording()
	#define FRAMEPRO_REGISTER_CONNECTION_CHANGED_CALLBACK(callback, context)				FramePro::RegisterConnectionChangedCallback(callback, context)
	#define FRAMEPRO_UNREGISTER_CONNECTION_CHANGED_CALLBACK(callback)						FramePro::UnregisterConnectionChangedcallback(callback)
	#define FRAMEPRO_SET_THREAD_PRIORITY(priority)											FramePro::SetThreadPriority(priority)
	#define FRAMEPRO_SET_THREAD_AFFINITY(affinity)											FramePro::SetThreadAffinity(affinity)
	#define FRAMEPRO_BLOCK_SOCKETS()														FramePro::BlockSockets()
	#define FRAMEPRO_UNBLOCK_SOCKETS()														FramePro::UnblockSockets()
	#define FRAMEPRO_CLEANUP_THREAD()														FramePro::CleanupThread()
	#define FRAMEPRO_THREAD_SCOPE(thread_name)												FramePro::ThreadScope framepro_thread_scope(thread_name)
	#define FRAMEPRO_LOG(message)															FramePro::Log(message)
	#define FRAMEPRO_COLOUR(r, g, b)														((FramePro::uint)((((FramePro::uint)((r)&0xff))<<16) | (((FramePro::uint)((g)&0xff))<<8) | ((b)&0xff)))
	#define FRAMEPRO_SET_CONDITIONAL_SCOPE_MIN_TIME(microseconds)							FramePro::SetConditionalScopeMinTimeInMicroseconds(microseconds)

	// scope macros
	#define FRAMEPRO_SCOPE()																FramePro::TimerScope FRAMEPRO_UNIQUE(timer_scope)(FRAMEPRO_FUNCTION_NAME_LITERAL "|" FRAMEPRO_SOURCE_STRING)
	#define FRAMEPRO_NAMED_SCOPE(name)														FramePro::TimerScope FRAMEPRO_UNIQUE(timer_scope)(name "|" FRAMEPRO_SOURCE_STRING)
	#define FRAMEPRO_NAMED_SCOPE_W(name)													FramePro::TimerScopeW FRAMEPRO_UNIQUE(timer_scope)(name L"|" FRAMEPRO_SOURCE_STRING_W)
	#define FRAMEPRO_ID_SCOPE(name_id)														FramePro::IdTimerScope FRAMEPRO_UNIQUE(timer_scope)(name_id, FRAMEPRO_SOURCE_STRING)
	#define FRAMEPRO_DYNAMIC_SCOPE(dynamic_string)											FramePro::IdTimerScope FRAMEPRO_UNIQUE(timer_scope)(FramePro::IsConnected() ? FramePro::RegisterString(dynamic_string) : -1, FRAMEPRO_SOURCE_STRING)
	#define FRAMEPRO_CONDITIONAL_SCOPE()													FramePro::ConditionalTimerScope FRAMEPRO_UNIQUE(timer_scope)(FRAMEPRO_FUNCTION_NAME_LITERAL "|" FRAMEPRO_SOURCE_STRING)
	#define FRAMEPRO_CONDITIONAL_ID_SCOPE(name)												FramePro::ConditionalTimerScopeId FRAMEPRO_UNIQUE(timer_scope)(name, FRAMEPRO_SOURCE_STRING)
	#define FRAMEPRO_CONDITIONAL_NAMED_SCOPE(name)											FramePro::ConditionalTimerScope FRAMEPRO_UNIQUE(timer_scope)(name "|" FRAMEPRO_SOURCE_STRING)
	#define FRAMEPRO_CONDITIONAL_NAMED_SCOPE_W(name)										FramePro::ConditionalTimerScopeW FRAMEPRO_UNIQUE(timer_scope)(name L"|" FRAMEPRO_SOURCE_STRING_W)
	#define FRAMEPRO_CONDITIONAL_BOOL_SCOPE(b)												FramePro::ConditionalBoolTimerScope FRAMEPRO_UNIQUE(timer_scope)(FRAMEPRO_FUNCTION_NAME_LITERAL "|" FRAMEPRO_SOURCE_STRING, (b))
	#define FRAMEPRO_CONDITIONAL_BOOL_ID_SCOPE(name, b)										FramePro::ConditionalBoolTimerScopeId FRAMEPRO_UNIQUE(timer_scope)(name, FRAMEPRO_SOURCE_STRING, (b))
	#define FRAMEPRO_CONDITIONAL_BOOL_NAMED_SCOPE(name, b)									FramePro::ConditionalBoolTimerScope FRAMEPRO_UNIQUE(timer_scope)(name "|" FRAMEPRO_SOURCE_STRING, (b))
	#define FRAMEPRO_CONDITIONAL_BOOL_NAMED_SCOPE_W(name, b)								FramePro::ConditionalBoolTimerScopeW FRAMEPRO_UNIQUE(timer_scope)(name L"|" FRAMEPRO_SOURCE_STRING_W, (b))
	#define FRAMEPRO_START_NAMED_SCOPE(name)												FramePro::int64 framepro_start_##name=0; MULTI_STATEMENT( if(FramePro::IsConnected()) { FRAMEPRO_GET_CLOCK_COUNT(framepro_start_##name); FramePro::StartScope(); } )
	#define FRAMEPRO_STOP_NAMED_SCOPE(name)													MULTI_STATEMENT( if(FramePro::IsConnected()) { FramePro::int64 framepro_end_##name; FRAMEPRO_GET_CLOCK_COUNT(framepro_end_##name); FramePro::AddTimeSpan(#name "|" FRAMEPRO_SOURCE_STRING, framepro_start_##name, framepro_end_##name); FramePro::StopScope(); } )
	#define FRAMEPRO_CONDITIONAL_START_SCOPE()												FramePro::int64 framepro_start=0; MULTI_STATEMENT( if(FramePro::IsConnected()) { FRAMEPRO_GET_CLOCK_COUNT(framepro_start); FramePro::StartScope(); } )
	#define FRAMEPRO_CONDITIONAL_STOP_NAMED_SCOPE(name)										MULTI_STATEMENT( if(FramePro::IsConnected()) { FramePro::int64 framepro_end; FRAMEPRO_GET_CLOCK_COUNT(framepro_end); if(framepro_end - framepro_start > FramePro::GetConditionalScopeMinTime()) FramePro::AddTimeSpan(name "|" FRAMEPRO_SOURCE_STRING, framepro_start, framepro_end); FramePro::StopScope();} )
	#define FRAMEPRO_CONDITIONAL_STOP_DYNAMIC_SCOPE(dynamic_string)							MULTI_STATEMENT( if(FramePro::IsConnected()) { FramePro::int64 framepro_end; FRAMEPRO_GET_CLOCK_COUNT(framepro_end); if(framepro_end - framepro_start > FramePro::GetConditionalScopeMinTime()) FramePro::AddTimeSpan(FramePro::RegisterString(dynamic_string), FRAMEPRO_SOURCE_STRING, framepro_start, framepro_end); FramePro::StopScope(); } )
	#define FRAMEPRO_CONDITIONAL_PARENT_SCOPE(name, callback, pre_duration, post_duration)	FramePro::ConditionalParentTimerScope FRAMEPRO_UNIQUE(timer_scope)(name, FRAMEPRO_SOURCE_STRING, callback, pre_duration, post_duration)
	#define FRAMEPRO_SET_SCOPE_COLOUR(name, colour)											MULTI_STATEMENT( FramePro::SetScopeColour(FramePro::RegisterString(name), colour); )
	#define FRAMEPRO_CALLSTACK()															FramePro::SendScopeCallstack()

	// idle scope macros
	#define FRAMEPRO_IDLE_SCOPE()															FramePro::TimerScope FRAMEPRO_UNIQUE(timer_scope)(FRAMEPRO_FUNCTION_NAME_LITERAL "|" FRAMEPRO_SOURCE_STRING_IDLE)
	#define FRAMEPRO_IDLE_NAMED_SCOPE(name)													FramePro::TimerScope FRAMEPRO_UNIQUE(timer_scope)(name "|" FRAMEPRO_SOURCE_STRING_IDLE)
	#define FRAMEPRO_IDLE_NAMED_SCOPE_W(name)												FramePro::TimerScopeW FRAMEPRO_UNIQUE(timer_scope)(name L"|" FRAMEPRO_SOURCE_STRING_IDLE_W)
	#define FRAMEPRO_IDLE_ID_SCOPE(name_id)													FramePro::IdTimerScope FRAMEPRO_UNIQUE(timer_scope)(name_id, FRAMEPRO_SOURCE_STRING_IDLE)
	#define FRAMEPRO_IDLE_DYNAMIC_SCOPE(dynamic_string)										FramePro::IdTimerScope FRAMEPRO_UNIQUE(timer_scope)(FramePro::IsConnected() ? FramePro::RegisterString(dynamic_string) : -1, FRAMEPRO_SOURCE_STRING_IDLE)
	#define FRAMEPRO_IDLE_CONDITIONAL_SCOPE()												FramePro::ConditionalTimerScope FRAMEPRO_UNIQUE(timer_scope)(FRAMEPRO_FUNCTION_NAME_LITERAL "|" FRAMEPRO_SOURCE_STRING_IDLE)
	#define FRAMEPRO_IDLE_CONDITIONAL_ID_SCOPE(name)										FramePro::ConditionalTimerScopeId FRAMEPRO_UNIQUE(timer_scope)(name, FRAMEPRO_SOURCE_STRING_IDLE)
	#define FRAMEPRO_IDLE_CONDITIONAL_NAMED_SCOPE(name)										FramePro::ConditionalTimerScope FRAMEPRO_UNIQUE(timer_scope)(name "|" FRAMEPRO_SOURCE_STRING_IDLE)
	#define FRAMEPRO_IDLE_CONDITIONAL_NAMED_SCOPE_W(name)									FramePro::ConditionalTimerScopeW FRAMEPRO_UNIQUE(timer_scope)(name L"|" FRAMEPRO_SOURCE_STRING_IDLE_W)
	#define FRAMEPRO_IDLE_START_NAMED_SCOPE(name)											FramePro::int64 framepro_start_##name=0; if(FramePro::IsConnected()) { FRAMEPRO_GET_CLOCK_COUNT(framepro_start_##name); FramePro::StartScope(); }
	#define FRAMEPRO_IDLE_STOP_NAMED_SCOPE(name)											MULTI_STATEMENT( if(FramePro::IsConnected()) { FramePro::int64 framepro_end_##name; FRAMEPRO_GET_CLOCK_COUNT(framepro_end_##name); FramePro::AddTimeSpan(#name "|" FRAMEPRO_SOURCE_STRING_IDLE, framepro_start_##name, framepro_end_##name); FramePro::StopScope(); } )
	#define FRAMEPRO_IDLE_CONDITIONAL_START_SCOPE()											FramePro::int64 framepro_start=0; MULTI_STATEMENT( if(FramePro::IsConnected()) { FRAMEPRO_GET_CLOCK_COUNT(framepro_start); FramePro::StartScope(); } )
	#define FRAMEPRO_IDLE_CONDITIONAL_STOP_NAMED_SCOPE(name)								MULTI_STATEMENT( if(FramePro::IsConnected()) { FramePro::int64 framepro_end; FRAMEPRO_GET_CLOCK_COUNT(framepro_end); if(framepro_end - framepro_start > FramePro::GetConditionalScopeMinTime()) FramePro::AddTimeSpan(name "|" FRAMEPRO_SOURCE_STRING_IDLE, framepro_start, framepro_end); FramePro::StopScope(); } )
	#define FRAMEPRO_IDLE_CONDITIONAL_STOP_DYNAMIC_SCOPE(dynamic_string)					MULTI_STATEMENT( if(FramePro::IsConnected()) { FramePro::int64 framepro_end; FRAMEPRO_GET_CLOCK_COUNT(framepro_end); if(framepro_end - framepro_start > FramePro::GetConditionalScopeMinTime()) FramePro::AddTimeSpan(FramePro::RegisterString(dynamic_string), FRAMEPRO_SOURCE_STRING_IDLE, framepro_start, framepro_end); FramePro::StopScope(); } )

	// custom stat macros
	#define FRAMEPRO_CUSTOM_STAT(name, value, graph, unit, colour)							MULTI_STATEMENT( if(FramePro::IsConnected()) FramePro::AddCustomStat(name, value, graph, unit, colour); )
	#define FRAMEPRO_DYNAMIC_CUSTOM_STAT(name, value, graph, unit, colour)					MULTI_STATEMENT( if(FramePro::IsConnected()) FramePro::AddCustomStat(FramePro::RegisterString(name), value, FramePro::RegisterString(graph), FramePro::RegisterString(unit), colour); )
	#define FRAMEPRO_SCOPE_CUSTOM_STAT(name, value, graph, unit, colour)					MULTI_STATEMENT( if(FramePro::IsConnected()) FramePro::SetScopeCustomStat(name, value, graph, unit, colour); )
	#define FRAMEPRO_SET_CUSTOM_STAT_GRAPH(name, graph)										MULTI_STATEMENT( FramePro::SetCustomStatGraph(FramePro::RegisterString(name), FramePro::RegisterString(graph)); )
	#define FRAMEPRO_SET_CUSTOM_STAT_UNIT(name, unit)										MULTI_STATEMENT( FramePro::SetCustomStatUnit(FramePro::RegisterString(name), FramePro::RegisterString(unit)); )
	#define FRAMEPRO_SET_CUSTOM_STAT_COLOUR(name, colour)									MULTI_STATEMENT( FramePro::SetCustomStatColour(FramePro::RegisterString(name), colour); )

	// high-res timers
	#define FRAMEPRO_HIRES_SCOPE(name)														FramePro::HiResTimerScope FRAMEPRO_UNIQUE(hires_scope)(name)

	// custom stat timers
	#define FRAMEPRO_DECL_CUSTOM_STAT_TIMER(name)											FramePro::CustomStatTimer g_FrameProCustomStatTimer##name(#name)
	#define FRAMEPRO_EXTERN_CUSTOM_STAT_TIMER(name)											extern FramePro::CustomStatTimer g_FrameProCustomStatTimer##name
	#define FRAMEPRO_CUSTOM_STAT_TIMER_SCOPE(name)											FramePro::CustomStatTimerScope FRAMEPRO_UNIQUE(timer_scope)(g_FrameProCustomStatTimer##name)

	// events
	#define FRAMEPRO_EVENT(name, colour)													FramePro::AddEvent(name, colour)

	// wait events
	#define FRAMEPRO_WAIT_EVENT_SCOPE(event_id)												FramePro::WaitEventScope FRAMEPRO_UNIQUE(timer_scope)((FramePro::int64)event_id)
	#define FRAMEPRO_TRIGGER_WAIT_EVENT(event_id)											FramePro::TriggerWaitEvent((FramePro::int64)event_id);
#else

	#define FRAMEPRO_FRAME_START()															((void)0)
	#define FRAMEPRO_SHUTDOWN()																((void)0)
	#define FRAMEPRO_SET_PORT(port)															((void)0)
	#define FRAMEPRO_SESSION_INFO(name, id)													((void)0)
	#define FRAMEPRO_SET_ALLOCATOR(p_allocator)												((void)0)
	#define FRAMEPRO_SET_THREAD_NAME(name)													((void)0)
	#define FRAMEPRO_SET_THREAD_ID_NAME(thread_id, name)									((void)0)
	#define FRAMEPRO_THREAD_ORDER(thread_name)												((void)0)
	#define FRAMEPRO_REGISTER_STRING(str)													0
	#define FRAMEPRO_START_RECORDING(filename, context_switches, callstacks, max_file_size)	((void)0)
	#define FRAMEPRO_STOP_RECORDING()														((void)0)
	#define FRAMEPRO_REGISTER_CONNECTION_CHANGED_CALLBACK(callback, context)				((void)0)
	#define FRAMEPRO_UNREGISTER_CONNECTION_CHANGED_CALLBACK(callback)						((void)0)
	#define FRAMEPRO_SET_THREAD_PRIORITY(priority)											((void)0)
	#define FRAMEPRO_SET_THREAD_AFFINITY(affinity)											((void)0)
	#define FRAMEPRO_UNBLOCK_SOCKETS()														((void)0)
	#define FRAMEPRO_CLEANUP_THREAD()														((void)0)
	#define FRAMEPRO_THREAD_SCOPE(thread_name)												((void)0)
	#define FRAMEPRO_LOG(message)															((void)0)
	#define FRAMEPRO_COLOUR(r, g, b)														((void)0)
	#define FRAMEPRO_SET_CONDITIONAL_SCOPE_MIN_TIME(microseconds)							((void)0)

	#define FRAMEPRO_SCOPE()																((void)0)
	#define FRAMEPRO_NAMED_SCOPE(name)														((void)0)
	#define FRAMEPRO_NAMED_SCOPE_W(name)													((void)0)
	#define FRAMEPRO_ID_SCOPE(name_id)														((void)name_id)
	#define FRAMEPRO_DYNAMIC_SCOPE(dynamic_string)											((void)0)
	#define FRAMEPRO_CONDITIONAL_SCOPE()													((void)0)
	#define FRAMEPRO_CONDITIONAL_ID_SCOPE(name)												((void)0)
	#define FRAMEPRO_CONDITIONAL_NAMED_SCOPE(name)											((void)0)
	#define FRAMEPRO_CONDITIONAL_NAMED_SCOPE_W(name)										((void)0)
	#define FRAMEPRO_CONDITIONAL_BOOL_SCOPE(b)												((void)0)
	#define FRAMEPRO_CONDITIONAL_BOOL_ID_SCOPE(name, b)										((void)0)
	#define FRAMEPRO_CONDITIONAL_BOOL_NAMED_SCOPE(name, b)									((void)0)
	#define FRAMEPRO_CONDITIONAL_BOOL_NAMED_SCOPE_W(name, b)								((void)0)
	#define FRAMEPRO_START_NAMED_SCOPE(name)												((void)0)
	#define FRAMEPRO_STOP_NAMED_SCOPE(name)													((void)0)
	#define FRAMEPRO_CONDITIONAL_START_SCOPE()												((void)0)
	#define FRAMEPRO_CONDITIONAL_STOP_NAMED_SCOPE(name)										((void)0)
	#define FRAMEPRO_CONDITIONAL_STOP_DYNAMIC_SCOPE(dynamic_string)							((void)0)
	#define FRAMEPRO_CONDITIONAL_PARENT_SCOPE(name, callback, pre_duration, post_duration)	((void)0)
	#define FRAMEPRO_SET_SCOPE_COLOUR(name, colour)											((void)0)
	#define FRAMEPRO_CALLSTACK()															((void)0)

	#define FRAMEPRO_IDLE_SCOPE()															((void)0)
	#define FRAMEPRO_IDLE_NAMED_SCOPE(name)													((void)0)
	#define FRAMEPRO_IDLE_NAMED_SCOPE_W(name)												((void)0)
	#define FRAMEPRO_IDLE_ID_SCOPE(name_id)													((void)name_id)
	#define FRAMEPRO_IDLE_DYNAMIC_SCOPE(dynamic_string)										((void)0)
	#define FRAMEPRO_IDLE_CONDITIONAL_SCOPE()												((void)0)
	#define FRAMEPRO_IDLE_CONDITIONAL_ID_SCOPE(name)										((void)0)
	#define FRAMEPRO_IDLE_CONDITIONAL_NAMED_SCOPE(name)										((void)0)
	#define FRAMEPRO_IDLE_CONDITIONAL_NAMED_SCOPE_W(name)									((void)0)
	#define FRAMEPRO_IDLE_START_NAMED_SCOPE(name)											((void)0)
	#define FRAMEPRO_IDLE_STOP_NAMED_SCOPE(name)											((void)0)
	#define FRAMEPRO_IDLE_CONDITIONAL_START_SCOPE()											((void)0)
	#define FRAMEPRO_IDLE_CONDITIONAL_STOP_NAMED_SCOPE(name)								((void)0)
	#define FRAMEPRO_IDLE_CONDITIONAL_STOP_DYNAMIC_SCOPE(dynamic_string)					((void)0)

	#define FRAMEPRO_CUSTOM_STAT(name, value, graph, unit, colour)							((void)0)
	#define FRAMEPRO_DYNAMIC_CUSTOM_STAT(name, value, graph, unit, colour)					((void)0)
	#define FRAMEPRO_SCOPE_CUSTOM_STAT(name, value, graph, unit, colour)					((void)0)
	#define FRAMEPRO_SET_CUSTOM_STAT_GRAPH(name, graph)										((void)0)
	#define FRAMEPRO_SET_CUSTOM_STAT_UNIT(name, unit)										((void)0)
	#define FRAMEPRO_SET_CUSTOM_STAT_COLOUR(name, colour)									((void)0)

	#define FRAMEPRO_HIRES_SCOPE(name)														((void)0)

	#define FRAMEPRO_DECL_CUSTOM_STAT_TIMER(name)											
	#define FRAMEPRO_EXTERN_CUSTOM_STAT_TIMER(name)
	#define FRAMEPRO_CUSTOM_STAT_TIMER_SCOPE(name)											((void)0)

	#define FRAMEPRO_EVENT(name, colour)													((void)0)

	#define FRAMEPRO_WAIT_EVENT_SCOPE(event_id)												((void)0)
	#define FRAMEPRO_TRIGGER_WAIT_EVENT(event_id)											((void)0)
#endif

//------------------------------------------------------------------------
// StringId typedef always needs to be defined
namespace FramePro
{
	typedef long long StringId;
}

//------------------------------------------------------------------------
// General defines

//------------------------------------------------------------------------
// Set this to zero if you don't want anyone to be able to connect with FramePro.
// Recording to a file from code is sill supported.
#define FRAMEPRO_SOCKETS_ENABLED 1

//------------------------------------------------------------------------
// Thread local storage buffers are flushed every 30 ms by default. You shouldn't need to change this.
#define FRAMEPRO_MAX_SEND_DELAY 30

//------------------------------------------------------------------------
// Write the network data out to a file. Only useful for debugging network issues.
#define FRAMEPRO_DEBUG_TCP 0

//------------------------------------------------------------------------
// FramePro will attempt to keep below this amount of memory. If the memory limit
// is reached it will stall your game. Nothing is pre-allocated, so FramePro will
// only use up as much mmeory as it needs, which depends on how many scopes are
// being sent per frame.
#define FRAMEPRO_MAX_MEMORY (50*1024*1024)

//------------------------------------------------------------------------
// For added security you can set this to true to disable all networking unless explicitly
// enabled from code using the FRAMEPRO_UNBLOCK_SOCKETS() macro. In general, it's best to completely
// define out FramePro with the FRAMEPRO_ENABLED define for retail builds.
#define FRAMEPRO_SOCKETS_BLOCKED_BY_DEFAULT false

//------------------------------------------------------------------------
// Never send scopes shorter than this. Undefine to send all scopes.
#define FRAMEPRO_SCOPE_MIN_TIME 10       // ns

//------------------------------------------------------------------------
// In order to not flood the channel with wait events of zero length, the event
// is only sent if it is longer than this value
#define FRAMEPRO_WAIT_EVENT_MIN_TIME 10		// ns

//------------------------------------------------------------------------
#define FRAMEPRO_ENABLE_CONTEXT_SWITCH_TRACKING 1

//------------------------------------------------------------------------
#define FRAMEPRO_ENABLE_CALLSTACKS 0

//------------------------------------------------------------------------
// If this is disabled and two dynamic strings happen to hash to the same value you may see
// incorrect strings in the FramePro app. Enabling this does add significant overhead
// to RegisterString. It is a good idea to use the string literal macros wherever possible
// which don't call RegiserString anyway.
#define FRAMEPRO_DETECT_HASH_COLLISIONS 0			

//------------------------------------------------------------------------
#define FRAMEPRO_STACK_TRACE_SIZE 128

//------------------------------------------------------------------------
// FramePro will assert if a hires scope doesn't have a parent scope. Enabling
// this define make the code detect that case and throw the hires scope away
// instead of asserting.
// This adds extra overhead to all scopes (function calls to increment/decrement
// on each start/end scope) and so if off by default
#define FRAMEPRO_ALLOW_UNPARENTED_HIRES_SCOPES 0

//------------------------------------------------------------------------
// DOCUMENTATION
//------------------------------------------------------------------------

// Setup
// -----
// Please see the setup guide in the help documentation or online, or the setup video for
// full instructions on setting up FramePro.
//

// Overview
// --------
// The scope macros are designed to have minimal overhead. Sending a scope to FramePro requires
// recording the start and end time of the scope and sending a string for the scope name. In order
// to avoid having to send strings across the network, FramePro only sends string ids. For string
// literals the id is the same as the string pointer and no further work needs to be done. The
// FramePro app will do the lookup and request the string value if necessary. For synamic strings
// the string has to be looked up in a hash map. For this reason it is always preferable to use
// the string literal versions of the macros.

// To send a scope the scope data (start, end, name_id) is copied to a thread local storage buffer.
// This means that there is no locking involved. The thread local storage buffers are flushed to the
// socket and sent across the network once every 30ms.

// Summary: Recording and sending a scope consists of grabbing the scope start and end time and copying
// those to a TLS buffer along with the name id (typically the string literal pointer) - and that's it!
// No locking, no hashmap lookups, minimal overhead.

// Platforms
// ---------
// FramePro supports many platforms, including Windows, Linux, XboxOne, PS4, UE4.
// If you need to add a platform please implement the functions in the Platform namespace (see FrameProPlatform.h)
// You will also need to define the platform specific defines in FrameProPlatform.h
// There are generic utility functions for Windows nad Linux in GenericPlatform that you can use.
// You can use EventTraceWin32 for context switch recording on windows based platforms.
// All platform specific code is in FrameProPlatform.cpp/h (or the end of the combined FramePro.cpp file)
//
// Proprietary comsole code is split out into seperate files, eg, FrameProPS4.cpp/h.
// Please contact slynch@puredevsoftware.com for these files.
//


// Definitions
// -----------
// string literal:		Normal C++ definition. A hard coded string value, for example "my_string"
// dynamic string:		A string built at run-time, for example std::string my_string = string1 + string2;
// Scope:				Normal meaning in the C++ context. A FramePro Scope is a struct that has a start/end time
//						and a name and appears in the thread and core views.
// Session:				A single profile. This can be either connected and the session data is sent to FramePro in realtime,
//						or the session can be recorded to a file.
// Recording:			The session data is written out to a file instead of being sent to FramePro over
//						the network.
// Conditional scope:	A scope that is only sent if the duration is above a threshold
// Dynamic scope:		A scope that takes a dynamic string as an argument (most macros only take string literals)


//--------------------------------------------------------------------------------------------------------
// Session Macros
//
// Session macros are macros that give general information about the session and threads, and
// start/stop recording sessions. The only macros that is required by FramePro is the FRAMEPRO_FRAME_START.
// It is also advised to set the thread name for each thread which is created.


// FRAMEPRO_FRAME_START()
// ----------------------
// This must be called at the start of each frame. It tells FramePro when the old frame ends and the new frame starts


// FRAMEPRO_SHUTDOWN()
// -------------------
// Cleans up all FramePro resources. No thread can call into FramePro after this is called.


// FRAMEPRO_SESSION_INFO(name, value)
// ----------------------------------
// name: string
// value: string
// The name/value pair will be displayed in the FramePro SessionInfo panel. Use it for things such as the build name, version, etc.


// FRAMEPRO_SET_ALLOCATOR(p_allocator)
// -----------------------------------
// p_allocator: FramePro::Allocator*
// Set the allocator that FramePro will use to allocate memory. This must be a class inherited from the FramePro::Allocator base class.
// The allocator class that you provide must be thread safe.
// NOTE: Call this before calling any other FramePro macro.


// FRAMEPRO_SET_THREAD_NAME(name)
// -----------------------------------------
// name: string (literal or dynamic)
// Tell FramePro the name of the current thread.
// FramePro will pick a colour based on the thread name which will always stay the same.


// FRAMEPRO_SET_THREAD_ID_NAME(thread_id, name)
// -----------------------------------------
// thread_id: id of the thread to set
// name: string (literal or dynamic)
// Tell FramePro the name of the specified thread.
// FramePro will pick a colour based on the thread name which will always stay the same.


// FRAMEPRO_THREAD_ORDER(thread_name)
// ----------------------------------
// thread_name: string (literal or dynamic)
// To keep the order of the threads in the thread view consistent for all users you can specify the thread ordering
// from your game. The threads will appear in the order that the macros are called. For example:
//		FRAMEPRO_THREAD_ORDER("MainThread");
//		FRAMEPRO_THREAD_ORDER("RenderTHread");
//		FRAMEPRO_THREAD_ORDER("WorkerThread1");
//		FRAMEPRO_THREAD_ORDER("WorkerThread2");
//		FRAMEPRO_THREAD_ORDER("WorkerThread3");
//		FRAMEPRO_THREAD_ORDER("WorkerThread4");


// FRAMEPRO_REGISTER_STRING(str)
// -----------------------------
// str: string (dynamic)
// returns: FramePro::StringId
// Passing in dynamic strings to scope macros is costly because the string has to be looked up ni a hashmap to
// determine if it needs to send the string value to FramePro. for dynamic strings that never change (for example
// the name of an entity) you can cache off the string id (in the entity) using this macro and pass that in to the
// scope instead of the string.


// FRAMEPRO_START_RECORDING(filename, context_switches, callstacks, max_file_size)
// -------------------------------------------------------------------
// filename: string - the filename to write the recording out to. Extension should be .framepro_recording
// context_switches: bool - whether to record context switches
// callstacks: bool - true to record the callstack of each scope. Warning, can negatively affect performance if you have a lot of scopes
// max_file_size: stop recording once the file reaches this size
// Instead of connecting to your game with FramePro you can instead write all of the network data out to a file.
// This recording file can then be loaded in FramePro. This can be useful if you want to start/stop a recording
// at a specific point in code, or if you want to automate the profiling session capture.
// If a recording was already started that recording will be stopped and this recording will be started.
// Connections from FramePro are disabled while a recording is in progress.


// FRAMEPRO_STOP_RECORDING()
// -------------------------
// Stop a recording started with FRAMEPRO_START_RECORDING.


// FRAMEPRO_REGISTER_CONNECTION_CHANGED_CALLBACK(callback, p_recording_filename, context)
// --------------------------------------------------------------------------------------
// callback: ConnectionChangedCallback function pointer
// context: void*
// Add a callback to be called when a connection is made or lost.
// context is passed in to the callback as user_data.
// The callback is of the form:
//		typedef void (*ConnectionChangedCallback)(bool connected, const wchar_t* p_recording_filename, void* user_data);
// If already connected the callback will be called immediately.
// Multiple callbacks can be registered.


// FRAMEPRO_UNREGISTER_CONNECTION_CHANGED_CALLBACK(callback)
// ---------------------------------------------------------
// Unregister a callback that was registered with FRAMEPRO_REGISTER_CONNECTION_CHANGED_CALLBACK


// FRAMEPRO_SET_THREAD_PRIORITY(priority)
// --------------------------------------
// priority: int
// Tell FramePro the priority of a thread. This will be displayed in the thread information (hover over the thread panel)
// It is often useful to be able to see the thread priorities in FramePro when looking at core usage.


// FRAMEPRO_SET_THREAD_AFFINITY(affinity)
// --------------------------------------
// affinity: int
// Tell FramePro the affinity of a thread. This will be displayed in the thread information (hover over the thread panel)
// It is often useful to be able to see the thread priorities in FramePro when looking at core usage.


// FRAMEPRO_UNBLOCK_SOCKETS()
// --------------------------
// As a security measure you can define FRAMEPRO_SOCKETS_BLOCKED_BY_DEFAULT to true. This will disable all network
// access by default. You can then call FRAMEPRO_UNBLOCK_SOCKETS() to explicitly allow connections.


// FRAMEPRO_CLEANUP_THREAD()
// --------------------------
// Call from a thread just before it exits to free the memory allocated for this thread. Do not call any other
// FramePro macros from the thread after calling this. FRAMEPRO_SHUTDOWN will automatically call this on all
// threads, so usually you don't need to call this. It is only necessary for threads that are created and destroyed.
// Note: Make sure that you don't call this inside a FramePro scope.
// during the session.


// FRAMEPRO_THREAD_SCOPE(thread_name)
// ----------------------------------
// thread_name: string (literal or dynamic)
// Calls FRAMEPRO_SET_THREAD_NAME at the start of the scope, and FRAMEPRO_CLEANUP_THREAD at the end of the scope.
// Ensure that this is called outside of any other scopes.


// FRAMEPRO_LOG(message)
// ---------------------
// message: string (literal or dynamic)
// Send a log message to FramePro. All log messages are timestamped.





//--------------------------------------------------------------------------------------------------------
// scope macros


// FRAMEPRO_SCOPE()
// ----------------
// Starts a timer immediately and stops it when the current scope ends. The scope name will be the function name.
// This is typically the most used scope. Put it at the start of a function.
// This macro has minimal overhead.


// FRAMEPRO_NAMED_SCOPE(name)
// --------------------------
// name: string - NOTE: MUST be string literal
// Starts a timer immediately and stops it when the current scope ends. This macro is used for timing any scope
// and allows you to explicitly set the scope name.
// This macro has minimal overhead.


// FRAMEPRO_NAMED_SCOPE_W(name)
// ----------------------------
// name: wide string - NOTE: MUST be string literal
// Starts a timer immediately and stops it when the current scope ends. This macro is used for timing any scope
// and allows you to explicitly set the scope name.
// This macro has minimal overhead.


// FRAMEPRO_ID_SCOPE(name_id)
// --------------------------
// name_id: FramePro::StringId
// Starts a timer immediately and stops it when the current scope ends. This macro is used for timing any scope
// and allows you to explicitly set the scope name. The name_id is a FramePro string is returned from FRAMEPRO_REGISTER_STRING
// This macro has minimal overhead.


// FRAMEPRO_DYNAMIC_SCOPE(dynamic_string)
// --------------------------------------
// dynamic_string: dynamic string
// Starts a timer immediately and stops it when the current scope ends. This macro is used for timing any scope
// and allows you to explicitly set the scope name.
// This macro has medium overhead beacuse it has to create a hash of the string and look it up on a hash map.
// Only use where absolutely necessary. In general, try and use the string literal or string id versions.


// FRAMEPRO_CONDITIONAL_SCOPE()
// ----------------------------
// Starts a timer immediately and stops it when the current scope ends. The scope name will be the function name.
// Put it at the start of a function. The data will only be sent to FramePro if the scope time is greater than the
// conditional scope threshold. The threshold is set in FramePro using the threshold slider and defaults to 50ms.
// Conditional scopes scan be useful for scopes that would be too numerous if always sent, but occasionally spike.


// FRAMEPRO_CONDITIONAL_ID_SCOPE(name)
// -----------------------------------
// name: FramePro::StringId
// Starts a timer immediately and stops it when the current scope ends. The name of the scope is set explicitly.
// The data will only be sent to FramePro if the scope time is greater than the conditional scope threshold. The
// threshold is set in FramePro using the threshold slider and defaults to 50ms.
// Conditional scopes scan be useful for scopes that would be too numerous if always sent, but occasionally spike.


// FRAMEPRO_CONDITIONAL_NAMED_SCOPE(name)
// --------------------------------------
// name: string - NOTE: MUST be string literal
// Starts a timer immediately and stops it when the current scope ends. The name of the scope is set explicitly.
// The data will only be sent to FramePro if the scope time is greater than the conditional scope threshold. The
// threshold is set in FramePro using the threshold slider and defaults to 50ms.
// Conditional scopes scan be useful for scopes that would be too numerous if always sent, but occasionally spike.


// FRAMEPRO_CONDITIONAL_NAMED_SCOPE_W(name)
// --------------------------------------
// name: string - NOTE: MUST be string literal
// Starts a timer immediately and stops it when the current scope ends. The name of the scope is set explicitly.
// The data will only be sent to FramePro if the scope time is greater than the conditional scope threshold. The
// threshold is set in FramePro using the threshold slider and defaults to 50ms.
// Conditional scopes scan be useful for scopes that would be too numerous if always sent, but occasionally spike.


// FRAMEPRO_CONDITIONAL_BOOL_SCOPE(b)
// ----------------------------------
// b: bool - whether to send the scope or not
// Starts a timer immediately and stops it when the current scope ends. The scope name will be the function name.
// Put it at the start of a function. The data will only be sent to FramePro if the scope time is greater than the
// conditional scope threshold. The threshold is set in FramePro using the threshold slider and defaults to 50ms.
// Conditional scopes scan be useful for scopes that would be too numerous if always sent, but occasionally spike.


// FRAMEPRO_CONDITIONAL_BOOL_ID_SCOPE(name, b)
// -------------------------------------------
// b: bool - whether to send the scope or not
// name: FramePro::StringId
// Starts a timer immediately and stops it when the current scope ends. The name of the scope is set explicitly.
// The data will only be sent to FramePro if the scope time is greater than the conditional scope threshold. The
// threshold is set in FramePro using the threshold slider and defaults to 50ms.
// Conditional scopes scan be useful for scopes that would be too numerous if always sent, but occasionally spike.


// FRAMEPRO_CONDITIONAL_BOOL_NAMED_SCOPE(name, b)
// ----------------------------------------------
// b: bool - whether to send the scope or not
// name: string - NOTE: MUST be string literal
// Starts a timer immediately and stops it when the current scope ends. The name of the scope is set explicitly.
// The data will only be sent to FramePro if the scope time is greater than the conditional scope threshold. The
// threshold is set in FramePro using the threshold slider and defaults to 50ms.
// Conditional scopes scan be useful for scopes that would be too numerous if always sent, but occasionally spike.


// FRAMEPRO_CONDITIONAL_NAMED_BOOL_SCOPE_W(name, b)
// ------------------------------------------------
// b: bool - whether to send the scope or not
// name: string - NOTE: MUST be string literal
// Starts a timer immediately and stops it when the current scope ends. The name of the scope is set explicitly.
// The data will only be sent to FramePro if the scope time is greater than the conditional scope threshold. The
// threshold is set in FramePro using the threshold slider and defaults to 50ms.
// Conditional scopes scan be useful for scopes that would be too numerous if always sent, but occasionally spike.


// FRAMEPRO_START_NAMED_SCOPE(name)
// --------------------------------------
// name: string literal without quotes - see comment
// name: string - NOTE: MUST be string literal
// Start a named scope. Creates a local variable which stores the start time of the scope.
// This must be matched by a call to FRAMEPRO_STOP_NAMED_SCOPE with the same name passed in.
// In general, it is preferable to use the scope marcos that automatically close the scope on exit, but
// there are occasions when you need to start/sto explicitly. Be careful not to overlap scopes, and be careful
// to always end the scope that you started. Otherwise you will see strange behavour in the thread stack view
// in FramePro.
// The name passed in is a string literal without the quotes, for example FRAMEPRO_START_NAMED_SCOPE(my_scope).
// This will create a local variable using called framepro_start_my_scope and send the scope using the "my_scope"
// string literal.


// FRAMEPRO_STOP_NAMED_SCOPE(name)
// -------------------------------
// name: string literal without quotes - this must be the same name that was passed in to FRAMEPRO_START_NAMED_SCOPE
// Stop a scope started with FRAMEPRO_START_NAMED_SCOPE


// FRAMEPRO_CONDITIONAL_START_SCOPE()
// ----------------------------------
// Starts a conditional scope. Note that you can only start one conditional scope per scope.
// Creates a local variable which stores the scope start time.


// FRAMEPRO_CONDITIONAL_STOP_NAMED_SCOPE(name)
// -------------------------------------------
// name: string - NOTE: MUST be string literal
// Stop a scope started with FRAMEPRO_CONDITIONAL_START_SCOPE. The name passed in is the name of the scope.
// The data will only be sent to FramePro if the scope time is greater than the conditional scope threshold. The
// threshold is set in FramePro using the threshold slider and defaults to 50ms.
// Conditional scopes scan be useful for scopes that would be too numerous if always sent, but occasionally spike.


// FRAMEPRO_CONDITIONAL_STOP_DYNAMIC_SCOPE(dynamic_string)
// -------------------------------------------------------
// name: string - dynamic string
// Stop a scope started with FRAMEPRO_CONDITIONAL_START_SCOPE. The name passed in is the name of the scope.
// The data will only be sent to FramePro if the scope time is greater than the conditional scope threshold. The
// threshold is set in FramePro using the threshold slider and defaults to 50ms.
// Conditional scopes scan be useful for scopes that would be too numerous if always sent, but occasionally spike.
// This scope is particularly useful for dynamic strings that are slow to evaluate. The string will only be
// evaluated if the scope time exceeds the threshold.
// For example: FRAMEPRO_CONDITIONAL_STOP_DYNAMIC_SCOPE(GetEntityName())
// Where GetEntityName() is too expensive to call all of the time, but if the scopes takes > 50ms it is worth
// calling it so that the name shows up in FramePro.


// FRAMEPRO_CONDITIONAL_PARENT_SCOPE(name, callback, pre_duration, post_duration)
// ------------------------------------------------------------------------------
// name: string literal - unique name of the parent
// callback: FramePro::ConditionalParentScopeCallback - called at the end of the parent scope. Return true form the callback to send the child scopes.
// pre_duration: int64 - how long to keep the child scopes if the callback returns false (in micro-sec). When the callback returns true all of these scopes will be sent.
// post_duration: int64 - how long to keep sending the child scopes after the callback has returned false (in micro-sec)
// The callback is called at the end of the scope. If the callback returns false
// the child scopes are not sent, instead they are added to a list. The
// child scopes are kept on the list for pre_duration us. If the
// callback reutrns true all of the child scopes for this frame and previous frames are sent,
// and the child scopes are continued to be sent for post_duration after the callback returns false.
// Note that nesting conditional parent scopes is not supported.


// FRAMEPRO_SET_SCOPE_COLOUR(name, colour)
// --------------------------------------
// name: string (literal or dynamic)
// colour: colour using FRAMEPRO_COLOUR macro
// Set the colour for a scope. Note this will have no effect if the user has set a custom colour in the UI
// This is a slow macro, so please only call once, not every frame.


// FRAMEPRO_CALLSTACK()
// --------------------------
// capture the callstack and send for the current scope


//--------------------------------------------------------------------------------------------------------
// idle scope macros

// All scope macros have an idle scope equivalent. An idle scope shows up as semi-transparent in FramePro.
// Separate macros were created to avoid having to add the overhead of an extra conditional to each scope.
// Idle scopes are typically used in Sleep or Event wait calls.


// FRAMEPRO_IDLE_SCOPE()
// ---------------------
// see FRAMEPRO_SCOPE

// FRAMEPRO_IDLE_NAMED_SCOPE(name)
// -------------------------------
// see FRAMEPRO_NAMED_SCOPE

// FRAMEPRO_IDLE_NAMED_SCOPE_W(name)
// ---------------------------------
// see FRAMEPRO_NAMED_SCOPE_W

// FRAMEPRO_IDLE_ID_SCOPE(name_id)
// -------------------------------
// see FRAMEPRO_ID_SCOPE

// FRAMEPRO_IDLE_DYNAMIC_SCOPE(dynamic_string)
// -------------------------------------------
// see FRAMEPRO_DYNAMIC_SCOPE

// FRAMEPRO_IDLE_CONDITIONAL_SCOPE()
// ---------------------------------
// see FRAMEPRO_CONDITIONAL_SCOPE

// FRAMEPRO_IDLE_CONDITIONAL_ID_SCOPE()
// ---------------------------------
// see FRAMEPRO_CONDITIONAL_SCOPE_ID

// FRAMEPRO_IDLE_CONDITIONAL_NAMED_SCOPE(name)
// -------------------------------------------
// see FRAMEPRO_CONDITIONAL_NAMED_SCOPE

// FRAMEPRO_IDLE_CONDITIONAL_NAMED_SCOPE_W(name)
// ---------------------------------------------
// see FRAMEPRO_CONDITIONAL_NAMED_SCOPE_W

// FRAMEPRO_IDLE_START_NAMED_SCOPE(name)
// -------------------------------------
// see FRAMEPRO_START_NAMED_SCOPE

// FRAMEPRO_IDLE_STOP_NAMED_SCOPE(name)
// ------------------------------------
// see FRAMEPRO_STOP_NAMED_SCOPE

// FRAMEPRO_IDLE_CONDITIONAL_START_SCOPE()
// ---------------------------------------
// see FRAMEPRO_CONDITIONAL_START_SCOPE

// FRAMEPRO_IDLE_CONDITIONAL_STOP_NAMED_SCOPE(name)
// ------------------------------------------------
// see FRAMEPRO_CONDITIONAL_STOP_NAMED_SCOPE

// FRAMEPRO_IDLE_CONDITIONAL_STOP_DYNAMIC_SCOPE(dynamic_string)
// ------------------------------------------------------------
// see FRAMEPRO_CONDITIONAL_STOP_DYNAMIC_SCOPE


//--------------------------------------------------------------------------------------------------------
// Custom Stat macros

// Custom Stats are zero'd at the start of the frame and accumulated during the frame. Each stat has an accumulated value
// and a count.The count is the number of times the stat was added to in a frame. Custom stats are keyed off the name
// value, which must be unique.The "graph" and "utils" values are only read on the first call for a stat and must stay
// the same thereafter. Graph is the graph name to display the stats. Multipole stats can be put into the same graph.
// The Units will show in the y-axis.

// FRAMEPRO_CUSTOM_STAT(name, value, graph, unit, colour)
// ------------------------------------------------------
// name: string literal							- the name of the custom stat
// value: int, int64, float, double				- value to add to the custom stat
// graph: string (literal or dynamic)			- the name of the graph in which the stat will be shown
// unit: string (literal or dynamic)			- the y-axis unit to be displayed in FramePro, eg "ms", "MB" etc for a custom stat
// colour: colour using FRAMEPRO_COLOUR macro	- the colour of the stat. Use zero for default colour. Note this will have no effect if the user has set a custom colour in the UI
// Adds value to the custom stat with the specified name

// FRAMEPRO_DYNAMIC_CUSTOM_STAT(name, value, graph, unit, colour)
// --------------------------------------------------------------
// name: dynamic string							- the name of the custom stat
// value: int, int64, float, double				- value to add to the custom stat
// graph: string (literal or dynamic)			- the name of the graph in which the stat will be shown
// unit: string (literal or dynamic)			- the y-axis unit to be displayed in FramePro, eg "ms", "MB" etc for a custom stat
// colour: colour using FRAMEPRO_COLOUR macro	- the colour of the stat. Use zero for default colour. Note this will have no effect if the user has set a custom colour in the UI
// Adds value to the custom stat with the specified name

// FRAMEPRO_SCOPE_CUSTOM_STAT(name, value, graph, unit, colour)
// ------------------------------------------------------------
// name: string literal or StringId - the name of the custom stat
// value: int64 - the stat value
// graph: string (literal or dynamic)			- the name of the graph in which the stat will be shown
// unit: string (literal or dynamic)			- the y-axis unit to be displayed in FramePro, eg "ms", "MB" etc for a custom stat
// colour: colour using FRAMEPRO_COLOUR macro	- the colour of the stat. Use zero for default colour. Note this will have no effect if the user has set a custom colour in the UI
// Add a custom stat to the current scope. This will be displayed in the scope hover box and in the frame graph view

// FRAMEPRO_SET_CUSTOM_STAT_GRAPH(name, graph)
// -------------------------------------------
// name: string (literal or dynamic)
// graph: string (literal or dynamic)
// Set the name of the graph in which to display the stat.

// FRAMEPRO_SET_CUSTOM_STAT_UNIT(name, unit)
// -----------------------------------------
// name: string (literal or dynamic)
// unit: string (literal or dynamic)
// Set the y-axis unit to be displayed in FramePro, eg "ms", "MB" etc for a custom stat. 

// FRAMEPRO_SET_CUSTOM_STAT_COLOUR(name, colour)
// ---------------------------------------------
// name: string (literal or dynamic)
// colour: colour using FRAMEPRO_COLOUR macro	- the colour of the stat. Use zero for default colour. Note this will have no effect if the user has set a custom colour in the UI


//--------------------------------------------------------------------------------------------------------
// hires timers

// FRAMEPRO_HIRES_SCOPE(name)
// --------------------------
// name: string literal
// Adds a hi-res timer for the current scope. Use hires timers as a low overhead timer for events that
// happen at a high frequency.
//
// Hires timers for a scope will be shown below that scope as a percentage of the total scope time. Start
// and end times are not recorded for hires timers, instead the total time and call count are accumulated
// for the current scope.
//
// For example, hires timer1 accounted for 60% of scope1 and was called 460 times. hires2 accounted
// for 10% of the time. The other 30% of time is spent in scope1.
// |---------------------------------------------------------------------------------------|
// |                                        scope1                                         |
// |---------------------------------------------------------------------------------------|
// |             hires1 60% (count: 460)                  |  hires2 10% (count: 80) |
// |--------------------------------------------------------------------------------|
//
// Hires timers are useful when you have something that you want to measure which happens at a very high
// frequency, such as memory allocations, string operations or updates for lots of entities. Because these
// sort of events happen so many times during a frame, adding normal scopes would have too much overhead
// and make the thread graph hard to read. Hires timers have minimal overhead and are clear and easy to see
// in the thread graph.



//--------------------------------------------------------------------------------------------------------
// custom stat timers

// Custom stat timers are very low overhead high resolution timers. A custom stat timer is simply
// a global timer value that is added to on every CUSTOM_STAT_TIMER_SCOPE. The timer is sent and
// reset at the end of a frame. Custom stat timers are useful for timing events that happen
// thousands of timer a frame.

// Usage:
//		// this defines a custom stat timer with name "my_timer"
//		FRAMEPRO_DECL_CUSTOM_STAT_TIMER(my_timer)
//
//		void MyFunction()
//		{
//			FRAMEPRO_CUSTOM_STAT_TIMER_SCOPE(my_timer)
//			...
//		}

// FRAMEPRO_DECL_CUSTOM_STAT_TIMER(name)
// -------------------------------------
// name: unquoted string		the name of the custom stat timer object
// Declares a custom stat timer with the specified name.

// FRAMEPRO_EXTERN_CUSTOM_STAT_TIMER(name)
// ---------------------------------------
// name: unquoted string		the name of the custom stat timer object
// Declares an extern to a custom stat timer with the specified name.

// FRAMEPRO_CUSTOM_STAT_TIMER_SCOPE(name)
// --------------------------------------
// name: unquoted string		the name of the custom stat timer
// Times the current scope and adds the value to the specified timer. The timer must have already been declared
// with FRAMEPRO_DECL_CUSTOM_STAT_TIMER.


//--------------------------------------------------------------------------------------------------------
// events

// FRAMEPRO_EVENT(name)
// --------------------
// name: string literal
// Add a named event which will be added to the timeilne.


//--------------------------------------------------------------------------------------------------------
// wait events

// Wait Events are not to be confused with Events. Wait Events are for measuring mutex or critical section
// locks. Use these macros to record how long your lock is held for, and who unlocked the event. Wait
// events will appear in the Cores View.

// FRAMEPRO_WAIT_EVENT_SCOPE(event_id)
// -----------------------------------
// event_id: unique identifier for the event (must be castable to int64)
// The easiest thing to use for the event_id is the this pointer of your Event or critical section class.
// Measure how long a thread waits on an event. For Win32 Events put this around the WaitForSingleObject
// call. For Win32 critical sections put this around the EnterCriticalSection call. In order to not flood
// the channel with events, the wait event is only sent if it is longer than FRAMEPRO_WAIT_EVENT_MIN_TIME.

// FRAMEPRO_TRIGGER_WAIT_EVENT(event_id)
// ----------------------------------
// event_id: unique identifier for the event (must be castable to int64).
// The event_id MUST match the id passed in to FRAMEPRO_WAIT_EVENT_SCOPE.
// Use the FRAMEPRO_TRIGGER_WAIT_EVENT to tell FramePro that a wait event has been triggered. For Win32
// events call _before_ the SetEvent call. For Win32 critical sections call just before the LeaveCriticalSection.



//------------------------------------------------------------------------
//
//                        ---- FramePro private ----
//
// In general do not use anything below this point. Stuff is only here in order ot keep everything inlined.
// The macros cover everything that you should need. Also, sticking to the macros means that FramePro can be
// completely defined out in retail builds.

//------------------------------------------------------------------------
#if FRAMEPRO_ENABLED

//------------------------------------------------------------------------
#include <stdlib.h>

//------------------------------------------------------------------------
// enable this if you are linking to FramePro.dll
//#define FRAMEPRO_DLL

//------------------------------------------------------------------------
// enable debug here to get FramePro asserts
#ifndef FRAMEPRO_DEBUG
	#if defined(DEBUG) || defined(_DEBUG)
		#define FRAMEPRO_DEBUG 1
	#else
		#define FRAMEPRO_DEBUG 0
	#endif
#endif

//------------------------------------------------------------------------
#define FRAMEPRO_STRINGIZE(x) FRAMEPRO_STRINGIZE2(x)
#define FRAMEPRO_STRINGIZE2(x) #x

//------------------------------------------------------------------------
#define FRAMEPRO_JOIN(a, b) FRAMEPRO_JOIN2(a, b)
#define FRAMEPRO_JOIN2(a, b) a##b
#define FRAMEPRO_UNIQUE(a) FRAMEPRO_JOIN(a, __LINE__)

//------------------------------------------------------------------------
#define FRAMEPRO_WIDESTR(s) FRAMEPRO_WIDESTR2(s)
#define FRAMEPRO_WIDESTR2(s) L##s

//------------------------------------------------------------------------
#if defined(__UNREAL__)
	#define FRAMEPRO_PLATFORM_UE4 1
	#define FRAMEPRO_PLATFORM_XBOXONE 0
	#define FRAMEPRO_PLATFORM_UWP 0
	#define FRAMEPRO_PLATFORM_WIN 0
	#define FRAMEPRO_PLATFORM_LINUX 0
	#define FRAMEPRO_PLATFORM_PS4 0
	#define FRAMEPRO_PLATFORM_ANDROID 0
#elif defined(_XBOX_ONE) || defined(_DURANGO)
	#define FRAMEPRO_PLATFORM_UE4 0
	#define FRAMEPRO_PLATFORM_XBOXONE 1
	#define FRAMEPRO_PLATFORM_UWP 0
	#define FRAMEPRO_PLATFORM_WIN 0
	#define FRAMEPRO_PLATFORM_LINUX 0
	#define FRAMEPRO_PLATFORM_PS4 0
	#define FRAMEPRO_PLATFORM_ANDROID 0
#elif defined(WINAPI_FAMILY) && (WINAPI_FAMILY == WINAPI_FAMILY_APP)
	#define FRAMEPRO_PLATFORM_UE4 0
	#define FRAMEPRO_PLATFORM_XBOXONE 0
	#define FRAMEPRO_PLATFORM_UWP 1
	#define FRAMEPRO_PLATFORM_WIN 0
	#define FRAMEPRO_PLATFORM_LINUX 0
	#define FRAMEPRO_PLATFORM_PS4 0
	#define FRAMEPRO_PLATFORM_ANDROID 0
#elif defined(_WIN32) || defined(_WIN64) || defined(WIN32) || defined(WIN64) || defined(__WIN32__) || defined(__WINDOWS__)
	#define FRAMEPRO_PLATFORM_UE4 0
	#define FRAMEPRO_PLATFORM_XBOXONE 0
	#define FRAMEPRO_PLATFORM_UWP 0
	#define FRAMEPRO_PLATFORM_WIN 1
	#define FRAMEPRO_PLATFORM_LINUX 0
	#define FRAMEPRO_PLATFORM_PS4 0
	#define FRAMEPRO_PLATFORM_ANDROID 0
#elif defined(__ORBIS__)
	#define FRAMEPRO_PLATFORM_UE4 0
	#define FRAMEPRO_PLATFORM_XBOXONE 0
	#define FRAMEPRO_PLATFORM_UWP 0
	#define FRAMEPRO_PLATFORM_WIN 0
	#define FRAMEPRO_PLATFORM_LINUX 0
	#define FRAMEPRO_PLATFORM_PS4 1
	#define FRAMEPRO_PLATFORM_ANDROID 0
#elif defined(__ANDROID__)
	#define FRAMEPRO_PLATFORM_UE4 0
	#define FRAMEPRO_PLATFORM_XBOXONE 0
	#define FRAMEPRO_PLATFORM_UWP 0
	#define FRAMEPRO_PLATFORM_WIN 0
	#define FRAMEPRO_PLATFORM_LINUX 0
	#define FRAMEPRO_PLATFORM_PS4 0
	#define FRAMEPRO_PLATFORM_ANDROID 1
#elif defined(unix) || defined(__unix__) || defined(__unix)
	#define FRAMEPRO_PLATFORM_UE4 0
	#define FRAMEPRO_PLATFORM_XBOXONE 0
	#define FRAMEPRO_PLATFORM_UWP 0
	#define FRAMEPRO_PLATFORM_WIN 0
	#define FRAMEPRO_PLATFORM_LINUX 1
	#define FRAMEPRO_PLATFORM_PS4 0
	#define FRAMEPRO_PLATFORM_ANDROID 0
#else
	#error FramePro platform not defined
#endif

//------------------------------------------------------------------------
#define FRAMEPRO_SOURCE_STRING			__FILE__ "|" FRAMEPRO_FUNCTION_NAME_LITERAL "|" FRAMEPRO_STRINGIZE(__LINE__) "|"
#define FRAMEPRO_SOURCE_STRING_W		FRAMEPRO_WIDESTR(__FILE__) L"|" FRAMEPRO_WIDESTR(FRAMEPRO_FUNCTION_NAME_LITERAL) L"|" FRAMEPRO_WIDESTR(FRAMEPRO_STRINGIZE(__LINE__)) L"|"
#define FRAMEPRO_SOURCE_STRING_IDLE		__FILE__ "|" FRAMEPRO_FUNCTION_NAME_LITERAL "|" FRAMEPRO_STRINGIZE(__LINE__) "|Idle"
#define FRAMEPRO_SOURCE_STRING_IDLE_W	FRAMEPRO_WIDESTR(__FILE__) L"|" FRAMEPRO_WIDESTR(FRAMEPRO_FUNCTION_NAME_LITERAL) L"|" FRAMEPRO_WIDESTR(FRAMEPRO_STRINGIZE(__LINE__)) L"|Idle"

//------------------------------------------------------------------------
#if FRAMEPRO_DEBUG
	#define FRAMEPRO_ASSERT(b) if(!(b)) FramePro::DebugBreak()
#else
	#define FRAMEPRO_ASSERT(b) ((void)0)
#endif

//------------------------------------------------------------------------
#define FRAMEPRO_UNREFERENCED(s) (void)s

//------------------------------------------------------------------------
// typedefs
namespace FramePro
{
	typedef long long int64;
	typedef unsigned long long uint64;
	typedef unsigned int uint;
}

//------------------------------------------------------------------------
#define FRAMEPRO_DEFAULT_COND_SCOPE_MIN_TIME 50

//------------------------------------------------------------------------
#define FRAMEPRO_MAX_INLINE_STRING_LENGTH 256

//------------------------------------------------------------------------
namespace FramePro
{
	typedef int(*ThreadMain)(void*);
}

//------------------------------------------------------------------------
namespace FramePro
{
	namespace PacketType
	{
		enum Enum
		{
			Connect = 0xaabb,
			FrameStart,
			TimeSpan,
			TimeSpanW,
			NamedTimeSpan,
			StringLiteralNamedTimeSpan,
			ThreadName,
			ThreadOrder,
			StringPacket,
			WStringPacket,
			NameAndSourceInfoPacket,
			NameAndSourceInfoPacketW,
			SourceInfoPacket,
			MainThreadPacket,
			RequestStringLiteralPacket,
			SetConditionalScopeMinTimePacket,
			ConnectResponsePacket,
			SessionStatsPacket,
			RequestRecordedDataPacket,
			legacy_SessionDetailsPacket,
			ContextSwitchPacket,
			ContextSwitchRecordingStartedPacket,
			ProcessNamePacket,
			CustomStatPacket_Depreciated,
			StringLiteralTimerNamePacket,
			HiResTimerScopePacket,
			LogPacket,
			EventPacket,
			StartWaitEventPacket,
			StopWaitEventPacket,
			TriggerWaitEventPacket,
			TimeSpanCustomStatPacket_Depreciated,
			TimeSpanWithCallstack,
			TimeSpanWWithCallstack,
			NamedTimeSpanWithCallstack,
			StringLiteralNamedTimeSpanWithCallstack,
			ModulePacket,
			SetCallstackRecordingEnabledPacket,
			CustomStatPacketW_Depreciated2,
			TimeSpanCustomStatPacketW,
			CustomStatPacket_Depreciated2,
			TimeSpanCustomStatPacket,
			ScopeColourPacket,
			CustomStatColourPacket,
			CustomStatGraphPacket,
			CustomStatUnitPacket,
			CallstackPacket,
			SessionInfoPacket,
			CustomStatPacket,
			CustomStatPacketW,
		};
	};
}

//------------------------------------------------------------------------
namespace FramePro
{
	struct ModulePacket
	{
		PacketType::Enum m_PacketType;
		int m_UseLookupFunctionForBaseAddress;
		int64 m_ModuleBase;
		char m_Sig[16];
		int m_Age;
		int m_Padding;
		char m_ModuleName[FRAMEPRO_MAX_INLINE_STRING_LENGTH];
		char m_SymbolFilename[FRAMEPRO_MAX_INLINE_STRING_LENGTH];
	};
}

//------------------------------------------------------------------------
namespace FramePro
{
	namespace Platform
	{
		enum Enum
		{
			Windows = 0,
			Windows_UWP,
			XBoxOne,
			Unused,
			Linux,
			PS4,
			Android,
			Mac,
			iOS,
			Switch,
		};
	}
}

//------------------------------------------------------------------------
// FRAMEPRO_API define
#ifndef FRAMEPRO_API
	#if defined(FRAMEPRO_DLL_EXPORT)
		#define FRAMEPRO_API __declspec(dllexport)
	#elif defined(FRAMEPRO_DLL)
		#define FRAMEPRO_API __declspec(dllimport)
	#else
		#define FRAMEPRO_API
	#endif
#endif

//------------------------------------------------------------------------

//------------------------------------------------------------------------
//
// FrameProPlatform.h
//
#if FRAMEPRO_ENABLED

//------------------------------------------------------------------------
// ---
// --- FRAMEPRO PLATFORM IMPLEMENTATION START ---
// ---
//------------------------------------------------------------------------

//------------------------------------------------------------------------
#include <stdarg.h> // IWYU pragma: export
#include <time.h> // IWYU pragma: export

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_UE4
//------------------------------------------------------------------------
#if FRAMEPRO_PLATFORM_UE4

	#include "FramePro/FrameProUE4.h" // contact slynch@puredevsoftware.com for this platform

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_WIN
//------------------------------------------------------------------------
#elif FRAMEPRO_PLATFORM_WIN

	FRAMEPRO_API __int64 FramePro_QueryPerformanceCounter();
	#define FRAMEPRO_GET_CLOCK_COUNT(time) time = FramePro_QueryPerformanceCounter()

	// Windows or Linux based platform
	#define FRAMEPRO_WIN_BASED_PLATFORM 1
	#define FRAMEPRO_LINUX_BASED_PLATFORM 0

	#define FRAMEPRO_USE_TLS_SLOTS 0

	// Port
	#define FRAMEPRO_PORT "8428"

	// x64 or x32
	#if defined(_WIN64) || defined(__LP64__) || defined(__x86_64__) || defined(__ppc64__)
		#define FRAMEPRO_X64 1
	#else
		#define FRAMEPRO_X64 0
	#endif

	#define FRAMEPRO_MAX_PATH 260

	#include <tchar.h>
	#define FRAMEPRO_TCHAR _TCHAR
	
	#define MULTI_STATEMENT(s) do { s } while(false)
	
	#if !defined(__clang__)
		#define FRAMEPRO_FUNCTION_DEFINE_IS_STRING_LITERAL 1
	#else
		#define FRAMEPRO_FUNCTION_DEFINE_IS_STRING_LITERAL 0
	#endif

	#define FRAMEPRO_NO_INLINE __declspec(noinline)
	#define FRAMEPRO_FORCE_INLINE __forceinline

	#define LIMIT_RECORDING_FILE_SIZE 1

	#define FRAMEPRO_ALIGN_STRUCT(a)

	#define FRAMEPRO_ENUMERATE_ALL_MODULES (FRAMEPRO_X64 && 1)

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_UWP
//------------------------------------------------------------------------
#elif FRAMEPRO_PLATFORM_UWP

	__int64 FramePro_QueryPerformanceCounter();
	#define FRAMEPRO_GET_CLOCK_COUNT(time) time = FramePro_QueryPerformanceCounter()

	// Windows or Linux based platform
	#define FRAMEPRO_WIN_BASED_PLATFORM 1
	#define FRAMEPRO_LINUX_BASED_PLATFORM 0

	#define FRAMEPRO_USE_TLS_SLOTS 0

	// Port
	#define FRAMEPRO_PORT "8428"

	// x64 or x32
	#define FRAMEPRO_X64 1

	#define FRAMEPRO_MAX_PATH 260

	#include <tchar.h>
	#define FRAMEPRO_TCHAR _TCHAR
	
	#define MULTI_STATEMENT(s) do { s } while(false)

	#if !defined(__clang__)
		#define FRAMEPRO_FUNCTION_DEFINE_IS_STRING_LITERAL 1
	#else
		#define FRAMEPRO_FUNCTION_DEFINE_IS_STRING_LITERAL 0
	#endif

	#define FRAMEPRO_NO_INLINE __declspec(noinline)
	#define FRAMEPRO_FORCE_INLINE __forceinline

	#define LIMIT_RECORDING_FILE_SIZE 1

	#define FRAMEPRO_ALIGN_STRUCT(a)

	#define FRAMEPRO_ENUMERATE_ALL_MODULES (FRAMEPRO_X64 && 1)

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_ANDROID
//------------------------------------------------------------------------
#elif FRAMEPRO_PLATFORM_ANDROID

	// this needs to be as fast as possible and inline
	#define FRAMEPRO_GET_CLOCK_COUNT(time)						\
		timespec FRAMEPRO_UNIQUE(ts);							\
		clock_gettime(CLOCK_MONOTONIC, &FRAMEPRO_UNIQUE(ts));	\
		time = FRAMEPRO_UNIQUE(ts).tv_sec * 1000000000LL + FRAMEPRO_UNIQUE(ts).tv_nsec

	// Windows or Linux based platform
	#define FRAMEPRO_WIN_BASED_PLATFORM 0
	#define FRAMEPRO_LINUX_BASED_PLATFORM 1

	#define FRAMEPRO_USE_TLS_SLOTS 0

	// Port
	#define FRAMEPRO_PORT "8428"

	// x64 or x32
	#if defined(__LP64__) || defined(__x86_64__) || defined(__ppc64__)
		#define FRAMEPRO_X64 1
	#else
		#define FRAMEPRO_X64 0
	#endif

	#define FRAMEPRO_MAX_PATH 260

	#define MULTI_STATEMENT(s) do { s } while(false)
	
	namespace FramePro { typedef char FRAMEPRO_TCHAR; }

	#define FRAMEPRO_FUNCTION_DEFINE_IS_STRING_LITERAL 0

	#define FRAMEPRO_NO_INLINE
	#define FRAMEPRO_FORCE_INLINE inline

	#define LIMIT_RECORDING_FILE_SIZE 1

	#define FRAMEPRO_ALIGN_STRUCT(a) __attribute__ ((aligned(a)))

	#define FRAMEPRO_ENUMERATE_ALL_MODULES (FRAMEPRO_X64 && 1)

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_LINUX
//------------------------------------------------------------------------
#elif FRAMEPRO_PLATFORM_LINUX

	// this needs to be as fast as possible and inline
	#define FRAMEPRO_GET_CLOCK_COUNT(time)						\
		timespec FRAMEPRO_UNIQUE(ts);							\
		clock_gettime(CLOCK_MONOTONIC, &FRAMEPRO_UNIQUE(ts));	\
		time = FRAMEPRO_UNIQUE(ts).tv_sec * 1000000000LL + FRAMEPRO_UNIQUE(ts).tv_nsec

	// Windows or Linux based platform
	#define FRAMEPRO_WIN_BASED_PLATFORM 0
	#define FRAMEPRO_LINUX_BASED_PLATFORM 1

	#define FRAMEPRO_USE_TLS_SLOTS 0

	// Port
	#define FRAMEPRO_PORT "8428"

	// x64 or x32
	#if defined(__LP64__) || defined(__x86_64__) || defined(__ppc64__)
		#define FRAMEPRO_X64 1
	#else
		#define FRAMEPRO_X64 0
	#endif

	#define FRAMEPRO_MAX_PATH 260

	#define MULTI_STATEMENT(s) do { s } while(false)
	
	namespace FramePro { typedef char FRAMEPRO_TCHAR; }

	#define FRAMEPRO_FUNCTION_DEFINE_IS_STRING_LITERAL 0

	#define FRAMEPRO_NO_INLINE
	#define FRAMEPRO_FORCE_INLINE inline

	#define LIMIT_RECORDING_FILE_SIZE 1

	#define FRAMEPRO_ALIGN_STRUCT(a) __attribute__ ((aligned(a)))

	#define FRAMEPRO_ENUMERATE_ALL_MODULES (FRAMEPRO_X64 && 1)

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_XBOXONE
//------------------------------------------------------------------------
#elif FRAMEPRO_PLATFORM_XBOXONE

	#include "FrameProXboxOne.h" // contact slynch@puredevsoftware.com for this platform

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_PS4
//------------------------------------------------------------------------
#elif FRAMEPRO_PLATFORM_PS4

	#include "FrameProPS4.h" // contact slynch@puredevsoftware.com for this platform

#else

	#error Platform not defined

#endif

//------------------------------------------------------------------------
// ---
// --- FRAMEPRO PLATFORM IMPLEMENTATION END ---
// ---
//------------------------------------------------------------------------

//------------------------------------------------------------------------
// check all necessary defines are set

#ifndef FRAMEPRO_GET_CLOCK_COUNT
	#error Platform must defined FRAMEPRO_GET_CLOCK_COUNT
#endif

#ifndef FRAMEPRO_PORT
	#error Platform must define FRAMEPRO_PORT
#endif

#ifndef FRAMEPRO_X64
	#error Platform must define FRAMEPRO_X64
#endif

#ifndef MULTI_STATEMENT
	#error Platform must define MULTI_STATEMENT
#endif

#ifndef FRAMEPRO_FUNCTION_DEFINE_IS_STRING_LITERAL
	#error Platform must define FRAMEPRO_FUNCTION_DEFINE_IS_STRING_LITERAL
#endif

#ifndef FRAMEPRO_NO_INLINE
	#error Platform must define FRAMEPRO_NO_INLINE
#endif

#ifndef FRAMEPRO_FORCE_INLINE
	#error Platform must define FRAMEPRO_FORCE_INLINE
#endif

#ifndef FRAMEPRO_USE_TLS_SLOTS
	#error Platform must define FRAMEPRO_USE_TLS_SLOTS
#endif

#ifndef LIMIT_RECORDING_FILE_SIZE
	#error Platform must define LIMIT_RECORDING_FILE_SIZE
#endif

//------------------------------------------------------------------------
namespace FramePro
{
	//------------------------------------------------------------------------
	class Allocator;
	struct ContextSwitch;
	class DynamicString;
	struct ModulePacket;
	template<typename T> class Array;

	//------------------------------------------------------------------------
	namespace Platform
	{
		typedef void (*ContextSwitchCallbackFunction)(const ContextSwitch&, void*);

		enum Enum;

		int64 GetTimerFrequency();

		void DebugBreak();

		int GetCore();

		bool GetProcessName(int process_id, char* p_name, int max_name_length);

		Platform::Enum GetPlatformEnum();

		void* CreateContextSwitchRecorder(Allocator* p_allocator);

		void DestroyContextSwitchRecorder(void* p_context_switch_recorder, Allocator* p_allocator);

		bool StartRecordingContextSitches(void* p_context_switch_recorder, ContextSwitchCallbackFunction p_callback, void* p_context, DynamicString& error, Allocator* p_allocator);
		
		void StopRecordingContextSitches(void* p_context_switch_recorder);

		void FlushContextSwitches(void* p_context_switch_recorder);

		void EnumerateModules(Array<ModulePacket*>& module_packets, Allocator* p_allocator);

		bool GetStackTrace(void** stack, int& stack_size, unsigned int& hash);

		int GetCurrentThreadId();

		void DebugWrite(const char* p_string);

		void GetLocalTime(tm* p_tm, const time_t *p_time);

		int GetCurrentProcessId();

		void VSPrintf(char* p_buffer, size_t const buffer_size, const char* p_format, va_list arg_list);

		void ToString(int value, char* p_dest, int dest_size);

		void CreateLock(void* p_os_lock_mem, int os_lock_mem_size);

		void DestroyLock(void* p_os_lock_mem);

		void TakeLock(void* p_os_lock_mem);

		void ReleaseLock(void* p_os_lock_mem);

		void CreateEventX(void* p_os_event_mem, int os_event_mem_size, bool initial_state, bool auto_reset);

		void DestroyEvent(void* p_os_event_mem);

		void SetEvent(void* p_os_event_mem);

		void ResetEvent(void* p_os_event_mem);

		int WaitEvent(void* p_os_event_mem, int timeout);

		bool InitialiseSocketSystem();

		void UninitialiseSocketSystem();

		void CreateSocket(void* p_os_socket_mem, int os_socket_mem_size);

		void DestroySocket(void* p_os_socket_mem);

		void DisconnectSocket(void* p_os_socket_mem, bool stop_listening);

		bool StartSocketListening(void* p_os_socket_mem);

		bool BindSocket(void* p_os_socket_mem, const char* p_port);

		bool AcceptSocket(void* p_source_os_socket_mem, void* p_target_os_socket_mem);

		bool SocketSend(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_sent);

		bool SocketReceive(void* p_os_socket_mem, const void* p_buffer, int size, int& bytes_received);

		bool IsSocketValid(const void* p_os_socket_mem);

		void HandleSocketError();

		bool CreateThread(
			void* p_os_thread_mem,
			int os_thread_mem_size,
			ThreadMain p_thread_main,
			void* p_context,
			Allocator* p_allocator);

		void DestroyThread(void* p_os_thread_mem);

		void SetThreadPriority(void* p_os_thread_mem, int priority);

		void SetThreadAffinity(void* p_os_thread_mem, int affinity);

		bool OpenFileForRead(void* p_os_file_mem, int os_file_mem_size, const char* p_filename);

		bool OpenFileForRead(void* p_os_file_mem, int os_file_mem_size, const wchar_t* p_filename);

		bool OpenFileForWrite(void* p_os_file_mem, int os_file_mem_size, const char* p_filename);

		bool OpenFileForWrite(void* p_os_file_mem, int os_file_mem_size, const wchar_t* p_filename);
		
		void CloseFile(void* p_os_file_mem);

		void ReadFromFile(void* p_os_file_mem, void* p_data, size_t size);

		void WriteToFile(void* p_os_file_mem, const void* p_data, size_t size);

		int GetFileSize(const void* p_os_file_mem);

		uint AllocateTLSSlot();

		void* GetTLSValue(uint slot);

		void SetTLSValue(uint slot, void* p_value);

		void GetRecordingFolder(char* p_path, int max_path_length);
	}
}

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_ENABLED
#include <atomic>

//------------------------------------------------------------------------
#if FRAMEPRO_WIN_BASED_PLATFORM
	#define FRAMEPRO_FUNCTION_NAME __FUNCSIG__
#else
	#define FRAMEPRO_FUNCTION_NAME __PRETTY_FUNCTION__
#endif

//------------------------------------------------------------------------
#if FRAMEPRO_FUNCTION_DEFINE_IS_STRING_LITERAL
	#define FRAMEPRO_FUNCTION_NAME_LITERAL FRAMEPRO_FUNCTION_NAME
#else
	#define FRAMEPRO_FUNCTION_NAME_LITERAL ""		// FRAMEPRO_FUNCTION_NAME is not a string literal on Linux platforms so we can't combine it with other string literals

	// If FRAMEPRO_FUNCTION_NAME is not a string literal it can't be
	// concatonated with other string literals so we need to pass in
	// the name separately.
	#ifdef FRAMEPRO_SCOPE
		#undef FRAMEPRO_SCOPE
		#define FRAMEPRO_SCOPE() 					FramePro::StringLiteralNamedTimerScope FRAMEPRO_UNIQUE(timer_scope)(FRAMEPRO_FUNCTION_NAME, FRAMEPRO_SOURCE_STRING)
	#endif
	#ifdef FRAMEPRO_CONDITIONAL_SCOPE
		#undef FRAMEPRO_CONDITIONAL_SCOPE
		#define FRAMEPRO_CONDITIONAL_SCOPE()		FramePro::StringLiteralNamedConditionalTimerScope FRAMEPRO_UNIQUE(timer_scope)(FRAMEPRO_FUNCTION_NAME, FRAMEPRO_SOURCE_STRING)
	#endif
	#ifdef FRAMEPRO_IDLE_SCOPE
		#undef FRAMEPRO_IDLE_SCOPE
		#define FRAMEPRO_IDLE_SCOPE()				FramePro::StringLiteralNamedTimerScope FRAMEPRO_UNIQUE(timer_scope)(FRAMEPRO_FUNCTION_NAME, FRAMEPRO_SOURCE_STRING_IDLE)
	#endif
	#ifdef FRAMEPRO_IDLE_CONDITIONAL_SCOPE
		#undef FRAMEPRO_IDLE_CONDITIONAL_SCOPE
		#define FRAMEPRO_IDLE_CONDITIONAL_SCOPE()	FramePro::StringLiteralNamedConditionalTimerScope FRAMEPRO_UNIQUE(timer_scope)(FRAMEPRO_FUNCTION_NAME, FRAMEPRO_SOURCE_STRING_IDLE)
	#endif
#endif

//------------------------------------------------------------------------
namespace FramePro
{
	FRAMEPRO_API void DebugBreak();

	// assume aligned reads/writes < 8 bytes are atomic on all platforms
	template<typename T>
	class RelaxedAtomic
	{
	public:
		RelaxedAtomic() { static_assert(sizeof(T) <= sizeof(size_t), "bad template arg"); FRAMEPRO_ASSERT((((FramePro::uint64)this) % sizeof(T) == 0)); }
		RelaxedAtomic(T value) : m_Value(value) { static_assert(sizeof(T) <= sizeof(size_t), "bad template arg"); FRAMEPRO_ASSERT((((FramePro::uint64)this) % sizeof(T) == 0)); }
		void operator=(T value) { m_Value = value; }
		operator T() const { return m_Value; }
	private:
		T m_Value;
	};
}

//------------------------------------------------------------------------
// the main FramePro interface
namespace FramePro
{
	// callbacks
	typedef void(*ConnectionChangedCallback)(bool connected, const wchar_t* p_recording_filename, void* user_data);
	typedef bool(*ConditionalParentScopeCallback)(const char* p_name, int64 start_time, int64 end_time, int64 ticks_per_second);

	FRAMEPRO_API extern RelaxedAtomic<bool> g_Connected;

	FRAMEPRO_API extern RelaxedAtomic<unsigned int> g_ConditionalScopeMinTime;

	inline bool IsConnected() { return g_Connected; }

	inline unsigned int GetConditionalScopeMinTime() { return g_ConditionalScopeMinTime; }

	FRAMEPRO_API void Shutdown();

	FRAMEPRO_API void SetPort(int port);

	FRAMEPRO_API void DebugBreak();

	FRAMEPRO_API void SendSessionInfo(const char* p_name, const char* p_value);

	FRAMEPRO_API void SendSessionInfo(const wchar_t* p_name, const wchar_t* p_value);

	FRAMEPRO_API void SetAllocator(class Allocator* p_allocator);		// if you call this you must call it BEFORE any other calls

	FRAMEPRO_API void FrameStart();		// call at the start of each of your frames

	FRAMEPRO_API void AddTimeSpan(const char* p_name_file_and_line, int64 start_time, int64 end_time);

	FRAMEPRO_API void AddTimeSpan(const wchar_t* p_name_file_and_line, int64 start_time, int64 end_time);

	FRAMEPRO_API void AddTimeSpan(const char* p_name, const char* p_file_and_line, int64 start_time, int64 end_time);

	FRAMEPRO_API void AddTimeSpan(StringId name_id, const char* p_file_and_line, int64 start_time, int64 end_time);

	FRAMEPRO_API void AddTimeSpan(StringId name_id, const char* p_file_and_line, int64 start_time, int64 end_time, int thread_id, int core);

	FRAMEPRO_API void AddCustomStat(const char* p_name, int value, const char* p_graph, const char* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(const char* p_name, int64 value, const char* p_graph, const char* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(const char* p_name, float value, const char* p_graph, const char* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(const char* p_name, double value, const char* p_graph, const char* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(const wchar_t* p_name, int value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(const wchar_t* p_name, int64 value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(const wchar_t* p_name, float value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(const wchar_t* p_name, double value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(StringId name, int value, const char* p_graph, const char* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(StringId name, int64 value, const char* p_graph, const char* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(StringId name, float value, const char* p_graph, const char* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(StringId name, double value, const char* p_graph, const char* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(StringId name, int value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(StringId name, int64 value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(StringId name, float value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(StringId name, double value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour);

	FRAMEPRO_API void AddCustomStat(StringId name, int value, StringId graph, StringId unit, uint colour);

	FRAMEPRO_API void AddCustomStat(StringId name, int64 value, StringId graph, StringId unit, uint colour);

	FRAMEPRO_API void AddCustomStat(StringId name, float value, StringId graph, StringId unit, uint colour);

	FRAMEPRO_API void AddCustomStat(StringId name, double value, StringId graph, StringId unit, uint colour);

	FRAMEPRO_API void AddEvent(const char* p_name, uint colour);

	FRAMEPRO_API void AddWaitEvent(int64 event_id, int64 start_time, int start_core, int64 end_time, int end_core);

	FRAMEPRO_API void TriggerWaitEvent(int64 event_id);

	FRAMEPRO_API void SetThreadName(const char* p_name);

	FRAMEPRO_API void SetThreadName(int thread_id, const char* p_name);

	FRAMEPRO_API void SetThreadOrder(StringId thread_name);

	FRAMEPRO_API StringId RegisterString(const char* p_str);

	FRAMEPRO_API StringId RegisterString(const wchar_t* p_str);

	FRAMEPRO_API void RegisterConnectionChangedCallback(ConnectionChangedCallback p_callback, void* p_context);

	FRAMEPRO_API void UnregisterConnectionChangedcallback(ConnectionChangedCallback p_callback);

	FRAMEPRO_API void StartRecording(const char* p_filename, bool context_switches, bool callstacks, int64 max_file_size);

	FRAMEPRO_API void StartRecording(const wchar_t* p_filename, bool context_switches, bool callstacks, int64 max_file_size);

	FRAMEPRO_API void StopRecording();

	FRAMEPRO_API void SetThreadPriority(int priority);

	FRAMEPRO_API void SetThreadAffinity(int affinity);

	FRAMEPRO_API void BlockSockets();

	FRAMEPRO_API void UnblockSockets();

	FRAMEPRO_API void AddCustomStatTimer(class CustomStatTimer* p_timer);

	FRAMEPRO_API void CleanupThread();

	FRAMEPRO_API void PushConditionalParentScope(const char* p_name, int64 pre_duration, int64 post_duration);		// durations are in us

	FRAMEPRO_API void PopConditionalParentScope(bool add_children);

	FRAMEPRO_API bool CallConditionalParentScopeCallback(ConditionalParentScopeCallback p_callback, const char* p_name, int64 start_time, int64 end_time);

	FRAMEPRO_API void StartHiResTimer(const char* p_name);

	FRAMEPRO_API void StopHiResTimer();

	FRAMEPRO_API void Log(const char* p_message);

	FRAMEPRO_API void SetScopeCustomStat(const char* p_name, int value, const char* p_graph, const char* p_unit, uint colour);
	
	FRAMEPRO_API void SetScopeCustomStat(const wchar_t* p_name, int value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour);

	FRAMEPRO_API void SetScopeCustomStat(StringId name, int value, StringId graph, StringId unit, uint colour);

	FRAMEPRO_API void SetScopeCustomStat(const char* p_name, int64 value, const char* p_graph, const char* p_unit, uint colour);

	FRAMEPRO_API void SetScopeCustomStat(const wchar_t* p_name, int64 value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour);

	FRAMEPRO_API void SetScopeCustomStat(StringId name, int64 value, StringId graph, StringId unit, uint colour);

	FRAMEPRO_API void SetScopeCustomStat(const char* p_name, float value, const char* p_graph, const char* p_unit, uint colour);

	FRAMEPRO_API void SetScopeCustomStat(const wchar_t* p_name, float value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour);

	FRAMEPRO_API void SetScopeCustomStat(StringId name, float value, StringId graph, StringId unit, uint colour);

	FRAMEPRO_API void SetScopeCustomStat(const char* p_name, double value, const char* p_graph, const char* p_unit, uint colour);

	FRAMEPRO_API void SetScopeCustomStat(const wchar_t* p_name, double value, const wchar_t* p_graph, const wchar_t* p_unit, uint colour);

	FRAMEPRO_API void SetScopeCustomStat(StringId name, double value, StringId graph, StringId unit, uint colour);

	FRAMEPRO_API void SetConditionalScopeMinTimeInMicroseconds(int64 value);

	FRAMEPRO_API void SetScopeColour(StringId name, uint colour);

	FRAMEPRO_API void SetCustomStatGraph(StringId name, StringId graph);

	FRAMEPRO_API void SetCustomStatUnit(StringId name, StringId unit);

	FRAMEPRO_API void SetCustomStatColour(StringId name, uint colour);

	FRAMEPRO_API void SendScopeCallstack();

	FRAMEPRO_API int GetCore();

#if FRAMEPRO_DEBUG || FRAMEPRO_ALLOW_UNPARENTED_HIRES_SCOPES
	FRAMEPRO_API void StartScope();
	FRAMEPRO_API void StopScope();
#else
	inline void StartScope() {}
	inline void StopScope() {}
#endif
}

//------------------------------------------------------------------------
// TimerScope
namespace FramePro
{
	//------------------------------------------------------------------------
	class Allocator
	{
	public:
		virtual void* Alloc(size_t size) = 0;
		virtual void Free(void* p) = 0;
		virtual ~Allocator() = default;
	};

	//------------------------------------------------------------------------
	class TimerScope
	{
	public:
		TimerScope(const char* p_name_and_source_info)
		{
			if (FramePro::IsConnected())
			{
				mp_NameAndSourceInfo = p_name_and_source_info;
				FRAMEPRO_GET_CLOCK_COUNT(m_StartTime);
				FramePro::StartScope();
			}
			else
			{
				mp_NameAndSourceInfo = nullptr;
				m_StartTime = 0;
			}
		}

		~TimerScope()
		{
			if(mp_NameAndSourceInfo)
			{
				int64 end_time = 0;
				FRAMEPRO_GET_CLOCK_COUNT(end_time);
				FramePro::AddTimeSpan(mp_NameAndSourceInfo, m_StartTime, end_time);
				FramePro::StopScope();
			}
		}

	private:
		const char* mp_NameAndSourceInfo;
		int64 m_StartTime;
	};

	//------------------------------------------------------------------------
	class TimerScopeW
	{
	public:
		TimerScopeW(const wchar_t* p_name_and_source_info)
		{
			if (FramePro::IsConnected())
			{
				mp_NameAndSourceInfo = p_name_and_source_info;
				FRAMEPRO_GET_CLOCK_COUNT(m_StartTime);
				FramePro::StartScope();
			}
			else
			{
				mp_NameAndSourceInfo = nullptr;
				m_StartTime = 0;
			}
		}

		~TimerScopeW()
		{
			if(mp_NameAndSourceInfo)
			{
				int64 end_time = 0;
				FRAMEPRO_GET_CLOCK_COUNT(end_time);
				FramePro::AddTimeSpan(mp_NameAndSourceInfo, m_StartTime, end_time);
				FramePro::StopScope();
			}
		}

	private:
		const wchar_t* mp_NameAndSourceInfo;
		int64 m_StartTime;
	};

	//------------------------------------------------------------------------
	class ConditionalParentTimerScope
	{
	public:
		ConditionalParentTimerScope(const char* p_name, const char* p_source_info, ConditionalParentScopeCallback p_callback, int64 pre_duration, int64 post_duration)
		:	mp_Callback(p_callback)
		{
			if (FramePro::IsConnected())
			{
				mp_Name = p_name;
				mp_SourceInfo = p_source_info;

				FramePro::PushConditionalParentScope(p_name, pre_duration, post_duration);

				FRAMEPRO_GET_CLOCK_COUNT(m_StartTime);

				FramePro::StartScope();
			}
			else
			{
				mp_Name = nullptr;
				mp_SourceInfo = nullptr;
				m_StartTime = 0;
			}
		}

		~ConditionalParentTimerScope()
		{
			if (mp_SourceInfo)
			{
				int64 end_time = 0;
				FRAMEPRO_GET_CLOCK_COUNT(end_time);

				bool add_children = FramePro::CallConditionalParentScopeCallback(mp_Callback, mp_Name, m_StartTime, end_time);
				FramePro::PopConditionalParentScope(add_children);

				FramePro::AddTimeSpan(mp_Name, mp_SourceInfo, m_StartTime, end_time);

				FramePro::StopScope();
			}
		}

	private:
		const char* mp_Name;
		const char* mp_SourceInfo;
		int64 m_StartTime;

		ConditionalParentScopeCallback mp_Callback;
	};

	//------------------------------------------------------------------------
	class IdTimerScope
	{
	public:
		IdTimerScope(StringId name, const char* p_source_info)
		{
			if (FramePro::IsConnected())
			{
				m_Name = name;
				mp_SourceInfo = p_source_info;
				FRAMEPRO_GET_CLOCK_COUNT(m_StartTime);
				FramePro::StartScope();
			}
			else
			{
				m_Name = 0;
				mp_SourceInfo = nullptr;
				m_StartTime = 0;
			}
		}

		~IdTimerScope()
		{
			if(mp_SourceInfo)
			{
				int64 end_time = 0;
				FRAMEPRO_GET_CLOCK_COUNT(end_time);
				FramePro::AddTimeSpan(m_Name, mp_SourceInfo, m_StartTime, end_time);
				FramePro::StopScope();
			}
		}

	private:
		StringId m_Name;
		const char* mp_SourceInfo;
		int64 m_StartTime;
	};

	//------------------------------------------------------------------------
	class StringLiteralNamedTimerScope
	{
	public:
		// p_name must be a string literal
		StringLiteralNamedTimerScope(const char* p_name, const char* p_source_info)
		{
			if (FramePro::IsConnected())
			{
				mp_Name = p_name;
				mp_SourceInfo = p_source_info;
				FRAMEPRO_GET_CLOCK_COUNT(m_StartTime);
				FramePro::StartScope();
			}
			else
			{
				mp_Name = nullptr;
				mp_SourceInfo = nullptr;
				m_StartTime = 0;
			}
		}

		~StringLiteralNamedTimerScope()
		{
			if(mp_SourceInfo)
			{
				int64 end_time = 0;
				FRAMEPRO_GET_CLOCK_COUNT(end_time);
				FramePro::AddTimeSpan(mp_Name, mp_SourceInfo, m_StartTime, end_time);
				FramePro::StopScope();
			}
		}

	private:
		const char* mp_Name;
		const char* mp_SourceInfo;
		int64 m_StartTime;
	};

	//------------------------------------------------------------------------
	class ConditionalTimerScope
	{
	public:
		ConditionalTimerScope(const char* p_name_and_source_info)
		{
			if (FramePro::IsConnected())
			{
				mp_NameAndSourceInfo = p_name_and_source_info;
				FRAMEPRO_GET_CLOCK_COUNT(m_StartTime);
				FramePro::StartScope();
			}
			else
			{
				mp_NameAndSourceInfo = nullptr;
				m_StartTime = 0;
			}
		}

		~ConditionalTimerScope()
		{
			if(mp_NameAndSourceInfo)
			{
				int64 end_time = 0;
				FRAMEPRO_GET_CLOCK_COUNT(end_time);
				if(end_time - m_StartTime > FramePro::GetConditionalScopeMinTime())
					FramePro::AddTimeSpan(mp_NameAndSourceInfo, m_StartTime, end_time);
				FramePro::StopScope();
			}
		}

	private:
		const char* mp_NameAndSourceInfo;
		int64 m_StartTime;
	};

	//------------------------------------------------------------------------
	class ConditionalTimerScopeId
	{
	public:
		ConditionalTimerScopeId(StringId name, const char* p_source_info)
		{
			if (FramePro::IsConnected())
			{
				m_Name = name;
				mp_SourceInfo = p_source_info;
				FRAMEPRO_GET_CLOCK_COUNT(m_StartTime);
				FramePro::StartScope();
			}
			else
			{
				m_Name = 0;
				mp_SourceInfo = nullptr;
				m_StartTime = 0;
			}
		}

		~ConditionalTimerScopeId()
		{
			if(mp_SourceInfo)
			{
				int64 end_time = 0;
				FRAMEPRO_GET_CLOCK_COUNT(end_time);
				if(end_time - m_StartTime > FramePro::GetConditionalScopeMinTime())
					FramePro::AddTimeSpan(m_Name, mp_SourceInfo, m_StartTime, end_time);
				FramePro::StopScope();
			}
		}

	private:
		StringId m_Name;
		const char* mp_SourceInfo;
		int64 m_StartTime;
	};

	//------------------------------------------------------------------------
	class ConditionalTimerScopeW
	{
	public:
		ConditionalTimerScopeW(const wchar_t* p_name_and_source_info)
		{
			if (FramePro::IsConnected())
			{
				mp_NameAndSourceInfo = p_name_and_source_info;
				FRAMEPRO_GET_CLOCK_COUNT(m_StartTime);
				FramePro::StartScope();
			}
			else
			{
				mp_NameAndSourceInfo = nullptr;
				m_StartTime = 0;
			}
		}

		~ConditionalTimerScopeW()
		{
			if(mp_NameAndSourceInfo)
			{
				int64 end_time = 0;
				FRAMEPRO_GET_CLOCK_COUNT(end_time);
				if(end_time - m_StartTime > FramePro::GetConditionalScopeMinTime())
					FramePro::AddTimeSpan(mp_NameAndSourceInfo, m_StartTime, end_time);
				FramePro::StopScope();
			}
		}

	private:
		const wchar_t* mp_NameAndSourceInfo;
		int64 m_StartTime;
	};

	//------------------------------------------------------------------------
	class ConditionalBoolTimerScope
	{
	public:
		ConditionalBoolTimerScope(const char* p_name_and_source_info, bool b)
		{
			if (b && FramePro::IsConnected())
			{
				mp_NameAndSourceInfo = p_name_and_source_info;
				FRAMEPRO_GET_CLOCK_COUNT(m_StartTime);
				FramePro::StartScope();
			}
			else
			{
				mp_NameAndSourceInfo = nullptr;
				m_StartTime = 0;
			}
		}

		~ConditionalBoolTimerScope()
		{
			if(mp_NameAndSourceInfo)
			{
				int64 end_time = 0;
				FRAMEPRO_GET_CLOCK_COUNT(end_time);
				if(end_time - m_StartTime > FramePro::GetConditionalScopeMinTime())
					FramePro::AddTimeSpan(mp_NameAndSourceInfo, m_StartTime, end_time);
				FramePro::StopScope();
			}
		}

	private:
		const char* mp_NameAndSourceInfo;
		int64 m_StartTime;
	};

	//------------------------------------------------------------------------
	class ConditionalBoolTimerScopeId
	{
	public:
		ConditionalBoolTimerScopeId(StringId name, const char* p_source_info, bool b)
		{
			if (b && FramePro::IsConnected())
			{
				m_Name = name;
				mp_SourceInfo = p_source_info;
				FRAMEPRO_GET_CLOCK_COUNT(m_StartTime);
				FramePro::StartScope();
			}
			else
			{
				m_Name = 0;
				mp_SourceInfo = nullptr;
				m_StartTime = 0;
			}
		}

		~ConditionalBoolTimerScopeId()
		{
			if(mp_SourceInfo)
			{
				int64 end_time = 0;
				FRAMEPRO_GET_CLOCK_COUNT(end_time);
				if(end_time - m_StartTime > FramePro::GetConditionalScopeMinTime())
					FramePro::AddTimeSpan(m_Name, mp_SourceInfo, m_StartTime, end_time);
				FramePro::StopScope();
			}
		}

	private:
		StringId m_Name;
		const char* mp_SourceInfo;
		int64 m_StartTime;
	};

	//------------------------------------------------------------------------
	class ConditionalBoolTimerScopeW
	{
	public:
		ConditionalBoolTimerScopeW(const wchar_t* p_name_and_source_info, bool b)
		{
			if (b && FramePro::IsConnected())
			{
				mp_NameAndSourceInfo = p_name_and_source_info;
				FRAMEPRO_GET_CLOCK_COUNT(m_StartTime);
				FramePro::StartScope();
			}
			else
			{
				mp_NameAndSourceInfo = nullptr;
				m_StartTime = 0;
			}
		}

		~ConditionalBoolTimerScopeW()
		{
			if(mp_NameAndSourceInfo)
			{
				int64 end_time = 0;
				FRAMEPRO_GET_CLOCK_COUNT(end_time);
				if(end_time - m_StartTime > FramePro::GetConditionalScopeMinTime())
					FramePro::AddTimeSpan(mp_NameAndSourceInfo, m_StartTime, end_time);
				FramePro::StopScope();
			}
		}

	private:
		const wchar_t* mp_NameAndSourceInfo;
		int64 m_StartTime;
	};

	//------------------------------------------------------------------------
	class StringLiteralNamedConditionalTimerScope
	{
	public:
		StringLiteralNamedConditionalTimerScope(const char* p_name, const char* p_source_info)
		{
			if (FramePro::IsConnected())
			{
				mp_Name = p_name;
				mp_SourceInfo = p_source_info;
				FRAMEPRO_GET_CLOCK_COUNT(m_StartTime);
				FramePro::StartScope();
			}
			else
			{
				mp_Name = nullptr;
				mp_SourceInfo = nullptr;
				m_StartTime = 0;
			}
		}

		~StringLiteralNamedConditionalTimerScope()
		{
			if(mp_SourceInfo)
			{
				int64 end_time = 0;
				FRAMEPRO_GET_CLOCK_COUNT(end_time);
				if(end_time - m_StartTime > FramePro::GetConditionalScopeMinTime())
					FramePro::AddTimeSpan(mp_Name, mp_SourceInfo, m_StartTime, end_time);
				FramePro::StopScope();
			}
		}

	private:
		const char* mp_Name;
		const char* mp_SourceInfo;
		int64 m_StartTime;
	};

	//------------------------------------------------------------------------
	class CustomStatTimer
	{
	public:
		CustomStatTimer(const char* p_name)
		:	m_Value(0),
			mp_Next(NULL)
		{
			mp_Name = p_name;
			FramePro::AddCustomStatTimer(this);
		}

		void Add(uint value)
		{
			if (mp_Name)
			{
				uint64 existing_value = m_Value.load(std::memory_order_relaxed);
				uint64 new_value;
				do
				{
					uint64 duration = existing_value & 0xffffffffff;
					uint64 count = (existing_value >> 40) & 0xffffff;
					duration += value;
					++count;
					new_value = (count << 40) | duration;

					FRAMEPRO_ASSERT(count <= 0xffffff);
					FRAMEPRO_ASSERT(duration <= 0xffffffffffL);

				} while (!m_Value.compare_exchange_weak(existing_value, new_value, std::memory_order_relaxed, std::memory_order_relaxed));
			}
		}

		void GetAndClear(uint64& value, uint& count)
		{
			uint64 existing_value = m_Value.load(std::memory_order_relaxed);
			while (!m_Value.compare_exchange_weak(existing_value, 0, std::memory_order_relaxed, std::memory_order_relaxed))
				;

			value = existing_value & 0xffffffffff;
			count = (existing_value >> 40) & 0xffffff;
		}

		void SetNext(CustomStatTimer* p_next) { mp_Next = p_next; }

		CustomStatTimer* GetNext() const { return mp_Next; }

		const char* GetName() const { return mp_Name; }

	private:
		std::atomic<uint64> m_Value;

		CustomStatTimer* mp_Next;

		const char* mp_Name;
	};

	//------------------------------------------------------------------------
	class CustomStatTimerScope
	{
	public:
		CustomStatTimerScope(CustomStatTimer& timer)
		{
			if (FramePro::IsConnected())
			{
				mp_Timer = &timer;
				FRAMEPRO_GET_CLOCK_COUNT(m_StartTime);
			}
			else
			{
				mp_Timer = nullptr;
			}
		}

		~CustomStatTimerScope()
		{
			if (mp_Timer)
			{
				int64 end_time;
				FRAMEPRO_GET_CLOCK_COUNT(end_time);

				mp_Timer->Add((uint)(end_time - m_StartTime));
			}
		}

	private:
		CustomStatTimerScope(const CustomStatTimerScope&);
		CustomStatTimerScope& operator=(const CustomStatTimerScope&);

		int64 m_StartTime;
		CustomStatTimer* mp_Timer;
	};

	//------------------------------------------------------------------------
	class HiResTimerScope
	{
	public:
		HiResTimerScope(const char* p_name)
		{
			bool connected = FramePro::IsConnected();
			m_Connected = connected;

			if (connected)
				FramePro::StartHiResTimer(p_name);
		}

		~HiResTimerScope()
		{
			if (m_Connected)
				FramePro::StopHiResTimer();
		}

	private:
		bool m_Connected;
	};

	//------------------------------------------------------------------------
	class ThreadScope
	{
	public:
		ThreadScope(const char* p_thread_name)
		{
			SetThreadName(p_thread_name);
		}

		~ThreadScope()
		{
			CleanupThread();
		}
	};

	//------------------------------------------------------------------------
	class WaitEventScope
	{
	public:
		WaitEventScope(int64 event_id)
		{
			if (FramePro::IsConnected())
			{
				m_EventId = event_id;
				m_StartCore = FramePro::GetCore();
				FRAMEPRO_GET_CLOCK_COUNT(m_StartTime);
			}
			else
			{
				m_EventId = 0;
				m_StartCore = 0;
				m_StartTime = 0;
			}
		}

		~WaitEventScope()
		{
			if (m_EventId)
			{
				int64 end_time;
				FRAMEPRO_GET_CLOCK_COUNT(end_time);

				FramePro::AddWaitEvent(m_EventId, m_StartTime, m_StartCore, end_time, FramePro::GetCore());
			}
		}

	private:
		int64 m_EventId;
		int64 m_StartTime;
		int m_StartCore;
	};

	//------------------------------------------------------------------------
	inline unsigned int GetHashAndStackSize(void** p_stack, int& stack_size)
	{
		#if FRAMEPRO_X64
			const unsigned int prime = 0x01000193;
			unsigned int hash = prime;
			stack_size = 0;
			void** p = p_stack;
			while(*p)
			{
				uint64 key = (uint64)(*p++);
				key = (~key) + (key << 18);
				key = key ^ (key >> 31);
				key = key * 21;
				key = key ^ (key >> 11);
				key = key + (key << 6);
				key = key ^ (key >> 22);
				hash = hash ^ (unsigned int)key;
				++stack_size;
			}

			return hash;
		#else
			const unsigned int prime = 0x01000193;
			unsigned int hash = prime;
			stack_size = 0;
			while(p_stack[stack_size])
			{
				hash = (hash * prime) ^ (unsigned int)p_stack[stack_size];
				++stack_size;
			}

			return hash;
		#endif
	}

	//------------------------------------------------------------------------
	namespace ThreadState
	{
		enum Enum
		{
			Initialized = 0,
			Ready,
			Running,
			Standby,
			Terminated,
			Waiting,
			Transition,
			DeferredReady,

			Reserved1,
			Reserved2,
			Reserved3,
			Reserved4,
			Reserved5,
			Reserved6,
			Reserved7,
			Reserved8,
			Reserved9,
			Reserved10,
			Reserved11,
			Reserved12,
		};
	}

	//------------------------------------------------------------------------
	namespace ThreadWaitReason
	{
		enum Enum
		{
			Executive = 0,
			FreePage,
			PageIn,
			PoolAllocation,
			DelayExecution,
			Suspended,
			UserRequest,
			WrExecutive,
			WrFreePage,
			WrPageIn,
			WrPoolAllocation,
			WrDelayExecution,
			WrSuspended,
			WrUserRequest,
			WrEventPair,
			WrQueue,
			WrLpcReceive,
			WrLpcReply,
			WrVirtualMemory,
			WrPageOut,
			WrRendezvous,
			WrKeyedEvent,
			WrTerminated,
			WrProcessInSwap,
			WrCpuRateControl,
			WrCalloutStack,
			WrKernel,
			WrResource,
			WrPushLock,
			WrMutex,
			WrQuantumEnd,
			WrDispatchInt,
			WrPreempted,
			WrYieldExecution,
			WrFastMutex,
			WrGuardedMutex,
			WrRundown,
			MaximumWaitReason,
		};
	}

	//------------------------------------------------------------------------
	struct ContextSwitch
	{
		int64 m_Timestamp;
		int m_ProcessId;
		int m_CPUId;
		int m_OldThreadId;
		int m_NewThreadId;
		ThreadState::Enum m_OldThreadState;
		ThreadWaitReason::Enum m_OldThreadWaitReason;
	};
}

//------------------------------------------------------------------------

//------------------------------------------------------------------------
//
// Array.h
//
#include <memory.h>

//------------------------------------------------------------------------
#if FRAMEPRO_ENABLED

//------------------------------------------------------------------------
namespace FramePro
{
	//--------------------------------------------------------------------
	template<typename T>
	class Array
	{
	public:
		//--------------------------------------------------------------------
		Array()
		:	mp_Array(NULL),
			m_Count(0),
			m_Capacity(0),
			mp_Allocator(NULL)
		{
		}

		//--------------------------------------------------------------------
		~Array()
		{
			FRAMEPRO_ASSERT(!mp_Array);
		}

		//--------------------------------------------------------------------
		int GetCount() const
		{
			return m_Count;
		}

		//--------------------------------------------------------------------
		void Clear()
		{
			if(mp_Array)
			{
				mp_Allocator->Free(mp_Array);
				mp_Array = NULL;
			}

			m_Count = 0;
			m_Capacity = 0;
		}

		//--------------------------------------------------------------------
		void ClearNoFree()
		{
			m_Count = 0;
		}

		//--------------------------------------------------------------------
		void SetAllocator(Allocator* p_allocator)
		{
			FRAMEPRO_ASSERT(mp_Allocator == p_allocator || !(mp_Allocator != NULL && p_allocator != NULL));
			mp_Allocator = p_allocator;
		}

		//--------------------------------------------------------------------
		bool Add(const T& value)
		{
			if (m_Count == m_Capacity)
			{
				if (!Grow())
					return false;
			}

			mp_Array[m_Count] = value;
			++m_Count;

			return true;
		}

		//--------------------------------------------------------------------
		const T& operator[](int index) const
		{
			FRAMEPRO_ASSERT(index >= 0 && index < m_Count);
			return mp_Array[index];
		}

		//--------------------------------------------------------------------
		T& operator[](int index)
		{
			FRAMEPRO_ASSERT(index >= 0 && index < m_Count);
			return mp_Array[index];
		}

		//--------------------------------------------------------------------
		void RemoveAt(int index)
		{
			FRAMEPRO_ASSERT(index >= 0 && index < m_Count);
		
			if(index < m_Count - 1)
				memmove(mp_Array + index, mp_Array + index + 1, (m_Count - 1 - index) * sizeof(T));

			--m_Count;
		}

		//--------------------------------------------------------------------
		T RemoveLast()
		{
			FRAMEPRO_ASSERT(m_Count);

			return mp_Array[--m_Count];
		}

		//--------------------------------------------------------------------
		bool Contains(const T& value) const
		{
			for(int i=0; i<m_Count; ++i)
				if(mp_Array[i] == value)
					return true;
			return false;
		}

		//--------------------------------------------------------------------
		bool Resize(int count)
		{
			int new_capacity = m_Count + count;

			if (new_capacity > m_Capacity)
			{
				m_Capacity = m_Capacity ? 2 * m_Capacity : 32;
				while (m_Capacity < new_capacity)
					m_Capacity *= 2;

				T* p_new_array = (T*)mp_Allocator->Alloc(sizeof(T)*m_Capacity);
				if (!p_new_array)
				{
					mp_Array = NULL;
					return false;
				}

				if (mp_Array)
					memcpy(p_new_array, mp_Array, sizeof(T)*m_Count);
				mp_Allocator->Free(mp_Array);
				mp_Array = p_new_array;
			}

			m_Count = count;

			return true;
		}

	private:
		//--------------------------------------------------------------------
		bool Grow()
		{
			m_Capacity = m_Capacity ? 2*m_Capacity : 32;
			
			T* p_new_array = (T*)mp_Allocator->Alloc(sizeof(T)*m_Capacity);
			if (!p_new_array)
			{
				mp_Array = NULL;
				return false;
			}

			if(mp_Array)
				memcpy(p_new_array, mp_Array, sizeof(T)*m_Count);

			mp_Allocator->Free(mp_Array);
			mp_Array = p_new_array;

			return true;
		}

		//------------------------------------------------------------------------
		// data
	private:
		T* mp_Array;
		
		int m_Count;
		int m_Capacity;

		Allocator* mp_Allocator;
	};
}

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_ENABLED

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_ENABLED

//------------------------------------------------------------------------
#endif		// #ifndef FRAMEPRO_H_INCLUDED

