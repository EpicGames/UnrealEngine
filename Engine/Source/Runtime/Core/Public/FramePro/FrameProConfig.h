/*
Copyright 2019 PureDev Software Limited

Permission to use, copy, modify, and/or distribute this software for any
purpose with or without fee is hereby granted, provided that the above
copyright notice and this permission notice appear in all copies.

THE SOFTWARE IS PROVIDED "AS IS" AND THE AUTHOR DISCLAIMS ALL WARRANTIES
WITH REGARD TO THIS SOFTWARE INCLUDING ALL IMPLIED WARRANTIES OF
MERCHANTABILITY AND FITNESS. IN NO EVENT SHALL THE AUTHOR BE LIABLE FOR
ANY SPECIAL, DIRECT, INDIRECT, OR CONSEQUENTIAL DAMAGES OR ANY DAMAGES
WHATSOEVER RESULTING FROM LOSS OF USE, DATA OR PROFITS, WHETHER IN AN
ACTION OF CONTRACT, NEGLIGENCE OR OTHER TORTIOUS ACTION, ARISING OUT OF
OR IN CONNECTION WITH THE USE OR PERFORMANCE OF THIS SOFTWARE.
*/

#pragma once

//@EPIC BEGIN: enable for engine / game, not external tools. engine/game enables it in UnrealBuildTool.
#ifndef FRAMEPRO_ENABLED
	// *** enable/disable FramePro here
	#ifdef __UNREAL__
		#define FRAMEPRO_ENABLED 0
	#else
		#define FRAMEPRO_ENABLED 1
	#endif
#endif
//@EPIC END
