// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FramePro/FramePro.h"

#if FRAMEPRO_ENABLED

/** Wrapper for FramePro  */
class FFrameProProfiler
{
public:
	static CORE_API void Initialize();
	static CORE_API void TearDown();

	/** Called to mark the start of each frame  */
	static CORE_API void FrameStart();

	/** Begin a named event */
	static CORE_API void PushEvent(); // Event with no name, expected to be named at the end
	static CORE_API void PushEvent(const TCHAR* Text);
	static CORE_API void PushEvent(const ANSICHAR* Text);

	/** End currently active named event */
	static CORE_API void PopEvent();
	static CORE_API void PopEvent(const TCHAR* Override);
	static CORE_API void PopEvent(const ANSICHAR* Override);

	static CORE_API void StartFrameProRecordingFromCommand(const TArray< FString >& Args);
	static CORE_API FString StartFrameProRecording(const FString& FilenameRoot, int32 MinScopeTime, bool bAppendDateTime=true);
	static CORE_API void StopFrameProRecording();

	static CORE_API bool IsFrameProRecording();
	static CORE_API bool IsThreadContextReady();
};

#endif // FRAMEPRO_ENABLED
