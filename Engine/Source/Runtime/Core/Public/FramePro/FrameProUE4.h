/*
	This software is provided 'as-is', without any express or implied warranty.
	In no event will the author(s) be held liable for any damages arising from
	the use of this software.

	Permission is granted to anyone to use this software for any purpose, including
	commercial applications, and to alter it and redistribute it freely, subject to
	the following restrictions:

	1. The origin of this software must not be misrepresented; you must not
	claim that you wrote the original software. If you use this software
	in a product, an acknowledgment in the product documentation would be
	appreciated but is not required.
	2. Altered source versions must be plainly marked as such, and must not be
	misrepresented as being the original software.
	3. This notice may not be removed or altered from any source distribution.

	Author: Stewart Lynch
	www.puredevsoftware.com
	slynch@puredevsoftware.com

	Add FramePro.cpp to your project to allow FramePro to communicate with your application.
*/

// HEADER_UNIT_SKIP - Not included directly

//------------------------------------------------------------------------
#include "FramePro.h" // HEADER_UNIT_IGNORE

//------------------------------------------------------------------------
//                         FRAMEPRO_PLATFORM_UE4
//------------------------------------------------------------------------
#if FRAMEPRO_ENABLED && FRAMEPRO_PLATFORM_UE4

	#ifndef FRAMEPRO_UE4_INCLUDED
	#define FRAMEPRO_UE4_INCLUDED

		#include "CoreTypes.h"
		#include "HAL/PlatformMisc.h"
		#include "HAL/PlatformTime.h"

		// this needs to be as fast as possible and inline
		#define FRAMEPRO_GET_CLOCK_COUNT(time) time = FPlatformTime::Cycles64()

		#undef FRAMEPRO_API
		#define FRAMEPRO_API CORE_API

		// Windows or Linux based platform
//@EPIC begin - allow external definition + NDA platforms removed
#if !defined(FRAMEPRO_WIN_BASED_PLATFORM)
		#define FRAMEPRO_WIN_BASED_PLATFORM (PLATFORM_WINDOWS)
#endif
#if !defined(FRAMEPRO_LINUX_BASED_PLATFORM)
		#define FRAMEPRO_LINUX_BASED_PLATFORM (!FRAMEPRO_WIN_BASED_PLATFORM)
#endif
//@EPIC: end

		#define FRAMEPRO_USE_TLS_SLOTS 1

		// Port
#if !defined(FRAMEPRO_PORT) //@EPIC begin - allow external definition + XBOXONE removed
		#define FRAMEPRO_PORT "8428"
#endif //@EPIC: end

		// x64 or x32
		#define FRAMEPRO_X64 1

		#define FRAMEPRO_MAX_PATH 260

		#if defined(_WIN32) || defined(_WIN64) || defined(WIN32) || defined(WIN64) || defined(__WIN32__) || defined(__WINDOWS__)
			#include <tchar.h>
			#define FRAMEPRO_TCHAR TCHAR
		#else
			#define FRAMEPRO_TCHAR char
		#endif
	
		#if FRAMEPRO_LINUX_BASED_PLATFORM
			#define MULTI_STATEMENT(s) do { s } while(false)
		#else
			#define MULTI_STATEMENT(s) do { s } while(false)
		#endif
	
		#if !defined(__clang__)
			#define FRAMEPRO_FUNCTION_DEFINE_IS_STRING_LITERAL 1
		#else
			#define FRAMEPRO_FUNCTION_DEFINE_IS_STRING_LITERAL 0
		#endif

		#define FRAMEPRO_NO_INLINE FORCENOINLINE
		#define FRAMEPRO_FORCE_INLINE FORCEINLINE

		// convenience function that takes an FString and no callstacks bool
		namespace FramePro { FRAMEPRO_API void StartRecording(const FString& filename, bool context_switches, int64 max_file_size); }

		#define LIMIT_RECORDING_FILE_SIZE 0

		#define FRAMEPRO_ALIGN_STRUCT(a) GCC_ALIGN(a)

		#define FRAMEPRO_ENUMERATE_ALL_MODULES 0

		#define FRAMEPRO_PS4_CONTEXT_SWITCH_RECORDING 0

	#endif		// #ifndef FRAMEPRO_UE4_INCLUDED

//------------------------------------------------------------------------
#endif		// #if FRAMEPRO_PLATFORM_UE4

