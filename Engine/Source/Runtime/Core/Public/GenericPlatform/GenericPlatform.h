// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//---------------------------------------------------------------------
// Utility for automatically setting up the pointer-sized integer type
//---------------------------------------------------------------------

template<typename T32BITS, typename T64BITS, int PointerSize>
struct SelectIntPointerType
{
	// nothing here are is it an error if the partial specializations fail
};

template<typename T32BITS, typename T64BITS>
struct SelectIntPointerType<T32BITS, T64BITS, 8>
{
	// Select the 64 bit type.
	typedef T64BITS TIntPointer;
};

template<typename T32BITS, typename T64BITS>
struct SelectIntPointerType<T32BITS, T64BITS, 4>
{
	// Select the 32 bit type.
	typedef T32BITS TIntPointer;
};

/**
* Generic types for almost all compilers and platforms
**/
struct FGenericPlatformTypes
{
	//~ Unsigned base types

	// 8-bit unsigned integer
	typedef unsigned char 		uint8;

	// 16-bit unsigned integer
	typedef unsigned short int	uint16;

	// 32-bit unsigned integer
	typedef unsigned int		uint32;

	// 64-bit unsigned integer
	typedef unsigned long long	uint64;

	//~ Signed base types.

	// 8-bit signed integer
	typedef	signed char			int8;

	// 16-bit signed integer
	typedef signed short int	int16;

	// 32-bit signed integer
	typedef signed int	 		int32;

	// 64-bit signed integer
	typedef signed long long	int64;

	//~ Character types

	// An ANSI character. 8-bit fixed-width representation of 7-bit characters.
	typedef char				ANSICHAR;

	// A wide character. In-memory only. ?-bit fixed-width representation of the platform's natural wide character set. Could be different sizes on different platforms.
	typedef wchar_t				WIDECHAR;

	// An 8-bit character type. In-memory only. 8-bit representation.
	using UTF8CHAR = char8_t;

	// A 16-bit character type. In-memory only.  16-bit representation. Should really be char16_t but making this the generic option is easier for compilers which don't fully support C++11 yet (i.e. MSVC).
	typedef uint16				CHAR16;		

	// A 32-bit character type. In-memory only. 32-bit representation.
	using UTF32CHAR = char32_t;

	// A switchable character. In-memory only. Either ANSICHAR or WIDECHAR, depending on a licensee's requirements.
	typedef WIDECHAR			TCHAR;

	// Unsigned int. The same size as a pointer.
	typedef SelectIntPointerType<uint32, uint64, sizeof(void*)>::TIntPointer UPTRINT;

	// Signed int. The same size as a pointer.
	typedef SelectIntPointerType<int32, int64, sizeof(void*)>::TIntPointer PTRINT;

	// Unsigned int. The same size as a pointer.
	typedef UPTRINT SIZE_T;

	// Signed int. The same size as a pointer.
	typedef PTRINT SSIZE_T;

	typedef int32					TYPE_OF_NULL;
	typedef decltype(nullptr)		TYPE_OF_NULLPTR;
};
