// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"

#define		MAKEAFFINITYMASK1(x)							((1<<x))
#define		MAKEAFFINITYMASK2(x,y)							((1<<x)+(1<<y))
#define		MAKEAFFINITYMASK3(x,y,z)						((1<<x)+(1<<y)+(1<<z))
#define		MAKEAFFINITYMASK4(w,x,y,z)						((1<<w)+(1<<x)+(1<<y)+(1<<z))
#define		MAKEAFFINITYMASK5(v,w,x,y,z)					((1<<v)+(1<<w)+(1<<x)+(1<<y)+(1<<z))
#define		MAKEAFFINITYMASK6(u,v,w,x,y,z)					((1<<u)+(1<<v)+(1<<w)+(1<<x)+(1<<y)+(1<<z))
#define		MAKEAFFINITYMASK7(t,u,v,w,x,y,z)				((1<<t)+(1<<u)+(1<<v)+(1<<w)+(1<<x)+(1<<y)+(1<<z))
#define		MAKEAFFINITYMASK8(s,t,u,v,w,x,y,z)				((1<<s)+(1<<t)+(1<<u)+(1<<v)+(1<<w)+(1<<x)+(1<<y)+(1<<z))
#define		MAKEAFFINITYMASK9(r,s,t,u,v,w,x,y,z)			((1<<r)+(1<<s)+(1<<t)+(1<<u)+(1<<v)+(1<<w)+(1<<x)+(1<<y)+(1<<z))
#define		MAKEAFFINITYMASK10(q,r,s,t,u,v,w,x,y,z)			((1<<q)+(1<<r)+(1<<s)+(1<<t)+(1<<u)+(1<<v)+(1<<w)+(1<<x)+(1<<y)+(1<<z))
#define		MAKEAFFINITYMASK11(p,q,r,s,t,u,v,w,x,y,z)		((1<<p)+(1<<q)+(1<<r)+(1<<s)+(1<<t)+(1<<u)+(1<<v)+(1<<w)+(1<<x)+(1<<y)+(1<<z))
#define		MAKEAFFINITYMASK12(o,p,q,r,s,t,u,v,w,x,y,z)		((1<<o)+(1<<p)+(1<<q)+(1<<r)+(1<<s)+(1<<t)+(1<<u)+(1<<v)+(1<<w)+(1<<x)+(1<<y)+(1<<z))
#define		MAKEAFFINITYMASK13(n,o,p,q,r,s,t,u,v,w,x,y,z)	((1<<n)+(1<<o)+(1<<p)+(1<<q)+(1<<r)+(1<<s)+(1<<t)+(1<<u)+(1<<v)+(1<<w)+(1<<x)+(1<<y)+(1<<z))

/**
* The list of enumerated thread priorities we support
*/
enum EThreadPriority
{
	TPri_Normal,
	TPri_AboveNormal,
	TPri_BelowNormal,
	TPri_Highest,
	TPri_Lowest,
	TPri_SlightlyBelowNormal,
	TPri_TimeCritical,
	TPri_Num,
};

enum class EThreadCreateFlags : int8
{
	None = 0,
	SMTExclusive = (1 << 0),
};

ENUM_CLASS_FLAGS(EThreadCreateFlags);

class FGenericPlatformAffinity
{
public:
	static const uint64 GetMainGameMask()
	{
		return 0xFFFFFFFFFFFFFFFF;
	}

	static const uint64 GetRenderingThreadMask()
	{
		return 0xFFFFFFFFFFFFFFFF;
	}

	static const uint64 GetRHIThreadMask()
	{
		return 0xFFFFFFFFFFFFFFFF;
	}

	static const uint64 GetRHIFrameOffsetThreadMask()
	{
		return 0xFFFFFFFFFFFFFFFF;
	}

	static const uint64 GetRTHeartBeatMask()
	{
		return 0xFFFFFFFFFFFFFFFF;
	}

	static const uint64 GetPoolThreadMask()
	{
		return 0xFFFFFFFFFFFFFFFF;
	}

	static const uint64 GetTaskGraphThreadMask()
	{
		return 0xFFFFFFFFFFFFFFFF;
	}

	static const uint64 GetAudioRenderThreadMask()
	{
		return 0xFFFFFFFFFFFFFFFF;
	}

	static const uint64 GetNoAffinityMask()
	{
		return 0xFFFFFFFFFFFFFFFF;
	}

	static const uint64 GetTaskGraphBackgroundTaskMask()
	{
		return 0xFFFFFFFFFFFFFFFF;
	}

	static const uint64 GetTaskGraphHighPriorityTaskMask()
	{
		return 0xFFFFFFFFFFFFFFFF;
	}

	static const uint64 GetAsyncLoadingThreadMask()
	{
		return 0xFFFFFFFFFFFFFFFF;
	}

	static const uint64 GetIoDispatcherThreadMask()
	{
		return 0xFFFFFFFFFFFFFFFF;
	}

	static const uint64 GetTraceThreadMask()
	{
		return 0xFFFFFFFFFFFFFFFF;
	}

	// @todo what do we think about having this as a function in this class? Should be make a whole new one? 
	// scrap it and force the priority like before?
	static EThreadPriority GetRenderingThreadPriority()
	{
		return TPri_Normal;
	}

	static EThreadCreateFlags GetRenderingThreadFlags()
	{
		return EThreadCreateFlags::None;
	}
	
	static EThreadPriority GetRHIThreadPriority()
	{
		return TPri_Normal;
	}

	static EThreadPriority GetGameThreadPriority()
	{
		return TPri_Normal;
	}

	static EThreadCreateFlags GetRHIThreadFlags()
	{
		return EThreadCreateFlags::None;
	}

	static EThreadPriority GetTaskThreadPriority()
	{
		return TPri_SlightlyBelowNormal;
	}

	static EThreadPriority GetTaskBPThreadPriority()
	{
		return TPri_BelowNormal;
	}
};


