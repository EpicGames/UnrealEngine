// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"


/** Helper struct used to hold 128-bit values, internally represented as two 64-bit integers. */
struct MS_ALIGN(16) FInt128 
{
	/** The low part of the 128-bit integer. */
	int64 Low;
	/** The high part of the 128-bit integer. */
	int64 High;
} 
GCC_ALIGN(16);


/**
* DEPRECATED! UE atomics are not maintained and potentially will be physically deprecated. Use std::atomic<T> for new code
*
* Generic implementation...you are required to implement at least FPlatformAtomics::InterlockedCompareExchange but they are listed here 
* to provide a base implementation in terms of FPlatformAtomics::InterlockedCompareExchange and easy copy-paste
**/
struct FGenericPlatformAtomics
{
#if 0	// can never use these since the actual implementation would have to be in a derived class as we are not using virtuals
		// provided here for reference
	/**
	 * Atomically increments the value pointed to and returns that to the caller
	 */
	static FORCEINLINE int32 InterlockedIncrement(volatile int32* Value)
	{
		int32 RetVal;
		do
		{
			RetVal = *Value;
		} while (InterlockedCompareExchange((int32*)Value, RetVal + 1, RetVal) != RetVal);
		return RetVal + 1;
	}

	/**
	 * Atomically increments the value pointed to and returns that to the caller
	 */
	static FORCEINLINE int64 InterlockedIncrement(volatile int64* Value)
	{
		int64 RetVal;
		do
		{
			RetVal = *Value;
		} while (InterlockedCompareExchange((int64*)Value, RetVal + 1, RetVal) != RetVal);
		return RetVal + 1;
	}

	/**
	 * Atomically decrements the value pointed to and returns that to the caller
	 */
	static FORCEINLINE int32 InterlockedDecrement(volatile int32* Value)
	{
		int32 RetVal;
		do
		{
			RetVal = *Value;
		} while (InterlockedCompareExchange((int32*)Value, RetVal - 1, RetVal) != RetVal);
		return RetVal - 1;
	}

	/**
	 * Atomically decrements the value pointed to and returns that to the caller
	 */
	static FORCEINLINE int64 InterlockedDecrement(volatile int64* Value)
	{
		int64 RetVal;
		do
		{
			RetVal = *Value;
		} while (InterlockedCompareExchange((int64*)Value, RetVal - 1, RetVal) != RetVal);
		return RetVal - 1;
	}

	/**
	 * Atomically adds the amount to the value pointed to and returns the old
	 * value to the caller
	 */
	static FORCEINLINE int32 InterlockedAdd(volatile int32* Value,int32 Amount)
	{
		int32 RetVal;
		do
		{
			RetVal = *Value;
		} while (InterlockedCompareExchange((int32*)Value, RetVal + Amount, RetVal) != RetVal);
		return RetVal;
	}

	/**
	 * Atomically adds the amount to the value pointed to and returns the old
	 * value to the caller
	 */
	static FORCEINLINE int64 InterlockedAdd(volatile int64* Value, int64 Amount)
	{
		int64 RetVal;
		do
		{
			RetVal = *Value;
		} while (InterlockedCompareExchange((int64*)Value, RetVal + Amount, RetVal) != RetVal);
		return RetVal;
	}

	/**
	 * Atomically swaps two values returning the original value to the caller
	 */
	static FORCEINLINE int32 InterlockedExchange(volatile int32* Value,int32 Exchange)
	{
		int32 RetVal;
		do
		{
			RetVal = *Value;
		} while (InterlockedCompareExchange((int32*)Value, Exchange, RetVal) != RetVal);
		return RetVal;
	}

	/**
	 * Atomically swaps two values returning the original value to the caller
	 */
	static FORCEINLINE int64 InterlockedExchange (volatile int64* Value, int64 Exchange)
	{
		int64 RetVal;
		do
		{
			RetVal = *Value;
		} while (InterlockedCompareExchange((int64*)Value, Exchange, RetVal) != RetVal);
		return RetVal;
	}

	/**
	 * Atomically swaps two pointers returning the original pointer to the caller
	 */
	static FORCEINLINE void* InterlockedExchangePtr( void*volatile* Dest, void* Exchange )
	{
		#error must implement
	}

	/**
	 * Atomically compares the value to comparand and replaces with the exchange
	 * value if they are equal and returns the original value
	 */
	static FORCEINLINE int8 InterlockedCompareExchange(volatile int8* Dest,int8 Exchange,int8 Comparand)
	{
		#error must implement
	}

	/**
	 * Atomically compares the value to comparand and replaces with the exchange
	 * value if they are equal and returns the original value
	 */
	static FORCEINLINE int16 InterlockedCompareExchange(volatile int16* Dest,int16 Exchange,int16 Comparand)
	{
		#error must implement
	}

	/**
	 * Atomically compares the value to comparand and replaces with the exchange
	 * value if they are equal and returns the original value
	 */
	static FORCEINLINE int32 InterlockedCompareExchange(volatile int32* Dest,int32 Exchange,int32 Comparand)
	{
		#error must implement
	}

	/**
	 * Atomically compares the value to comparand and replaces with the exchange
	 * value if they are equal and returns the original value
	 * REQUIRED, even on 32 bit architectures. 
	 */
	static FORCEINLINE int64 InterlockedCompareExchange (volatile int64* Dest, int64 Exchange, int64 Comparand)
	{
		#error must implement
	}

	/**
	 * Atomically ands AndValue to the value pointed to and returns the old
	 * value to the caller. Implemented for int8, int16 and int64 as well.
	 */
	static FORCEINLINE int32 InterlockedAnd(volatile int32* Value, const int32 AndValue)
	{
		int32 RetVal;
		do
		{
			RetVal = *Value;
		} while (InterlockedCompareExchange((int32*)Value, RetVal & AndValue, RetVal) != RetVal);
		return RetVal;
	}

	/**
	 * Atomically ors OrValue to the value pointed to and returns the old
	 * value to the caller. Implemented for int8, int16 and int64 as well.
	 */
	static FORCEINLINE int32 InterlockedOr(volatile int32* Value, const int32 OrValue)
	{
		int32 RetVal;
		do
		{
			RetVal = *Value;
		} while (InterlockedCompareExchange((int32*)Value, RetVal | OrValue, RetVal) != RetVal);
		return RetVal;
	}

	/**
	 * Atomically xors XorValue to the value pointed to and returns the old
	 * value to the caller. Implemented for int8, int16 and int64 as well.
	 */
	static FORCEINLINE int32 InterlockedXor(volatile int32* Value, const int32 XorValue)
	{
		int32 RetVal;
		do
		{
			RetVal = *Value;
		} while (InterlockedCompareExchange((int32*)Value, RetVal ^ XorValue, RetVal) != RetVal);
		return RetVal;
	}

	/**
	 * Atomic read of 32 bit value with an implicit memory barrier.
	 */
	static FORCEINLINE int8 AtomicRead(volatile const int8* Src)
	{
		return InterlockedCompareExchange((volatile int8*)Src, 0, 0);
	}

	/**
	 * Atomic read of 32 bit value with an implicit memory barrier.
	 */
	static FORCEINLINE int16 AtomicRead(volatile const int16* Src)
	{
		return InterlockedCompareExchange((volatile int16*)Src, 0, 0);
	}

	/**
	 * Atomic read of 32 bit value with an implicit memory barrier.
	 */
	static FORCEINLINE int32 AtomicRead(volatile const int32* Src)
	{
		return InterlockedCompareExchange((volatile int32*)Src, 0, 0);
	}

	/**
	 * Atomic read of 32 bit value with an implicit memory barrier.
	 */
	static FORCEINLINE int64 AtomicRead(volatile const int64* Src)
	{
		return InterlockedCompareExchange((volatile int64*)Src, 0, 0);
	}

	/**
	 * Atomically compares the pointer to comparand and replaces with the exchange
	 * pointer if they are equal and returns the original value
	 */
	static FORCEINLINE void* InterlockedCompareExchangePointer(void*volatile* Dest,void* Exchange,void* Comperand)
	{
		#error must implement
	}

#if	PLATFORM_HAS_128BIT_ATOMICS
	/**
	 *	The function compares the Destination value with the Comparand value:
	 *		- If the Destination value is equal to the Comparand value, the Exchange value is stored in the address specified by Destination, 
	 *		- Otherwise, the initial value of the Destination parameter is stored in the address specified specified by Comparand.
	 *	
	 *	@return true if Comparand equals the original value of the Destination parameter, or false if Comparand does not equal the original value of the Destination parameter.
	 *	
	 */
	static FORCEINLINE bool InterlockedCompareExchange128( volatile FInt128* Dest, const FInt128& Exchange, FInt128* Comparand )
	{
		#error Must implement
	}

	/**
	* Atomic read of 128 bit value with a memory barrier
	*/
	static FORCEINLINE void AtomicRead128(const volatile FInt128* Src, FInt128* OutResult)
	{
		FInt128 Zero;
		Zero.High = 0;
		Zero.Low = 0;
		*OutResult = Zero;
		InterlockedCompareExchange128((volatile FInt128*)Src, Zero, OutResult);
	}
#endif // PLATFORM_HAS_128BIT_ATOMICS

#endif // 0


	/**
	 * @return true, if the processor we are running on can execute compare and exchange 128-bit operation.
	 * @see cmpxchg16b, early AMD64 processors don't support this operation.
	 */
	static FORCEINLINE bool CanUseCompareExchange128()
	{
		return false;
	}

protected:
	/**
	 * Checks if a pointer is aligned and can be used with atomic functions.
	 *
	 * @param Ptr - The pointer to check.
	 *
	 * @return true if the pointer is aligned, false otherwise.
	 */
	static inline bool IsAligned( const volatile void* Ptr, const uint32 Alignment = sizeof(void*) )
	{
		return !(PTRINT(Ptr) & (Alignment - 1));
	}
};

