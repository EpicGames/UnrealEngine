// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	GenericPlatformChunkInstall.h: Generic platform chunk based install classes.
==============================================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Templates/Function.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "UObject/NameTypes.h"

class IPlatformChunkInstall;

CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogChunkInstaller, Log, All);

#ifndef ENABLE_PLATFORM_CHUNK_INSTALL
	#define ENABLE_PLATFORM_CHUNK_INSTALL (1)
#endif

namespace EChunkLocation
{
	enum Type
	{
		// note: higher quality locations should have higher enum values, we sort by these in AssetRegistry.cpp
		DoesNotExist,	// chunk does not exist
		NotAvailable,	// chunk has not been installed yet
		LocalSlow,		// chunk is on local slow media (optical)
		LocalFast,		// chunk is on local fast media (HDD)

		BestLocation=LocalFast
	};
}


namespace EChunkInstallSpeed
{
	enum Type
	{
		Paused,					// chunk installation is paused
		Slow,					// installation is lower priority than Game IO
		Fast					// installation is higher priority than Game IO
	};
}

namespace EChunkPriority
{
	enum Type
	{
		Immediate,	// Chunk install is of highest priority, this can cancel lower priority installs.
		High	 ,	// Chunk is probably required soon so grab is as soon as possible.
		Low		 ,	// Install this chunk only when other chunks are not needed.
	};
}

namespace EChunkProgressReportingType
{
	enum Type
	{
		ETA,					// time remaining in seconds
		PercentageComplete		// percentage complete in 99.99 format
	};
}

struct FNamedChunkCompleteCallbackParam
{
	FName NamedChunk;
	EChunkLocation::Type Location;
	bool bIsInstalled;
	bool bHasSucceeded;
};

/**
 * Platform Chunk Install Module Interface
 */
class IPlatformChunkInstallModule : public IModuleInterface
{
public:

	virtual IPlatformChunkInstall* GetPlatformChunkInstall() = 0;
};

/**
 * Platform Chunk Install Manifest Interface
 */
class IPlatformChunkInstallManifest
{
public:
	virtual bool HasManifest() const = 0;
	virtual int32 GetChunkIDFromPakchunkIndex(int32 PakchunkIndex) const = 0;
	virtual TArray<FString> GetPakFilesInChunk(int32 ChunkID) const = 0;
};


/** Delegate called when a chunk either successfully installs or fails to install, bool is success */
DECLARE_DELEGATE_TwoParams(FPlatformChunkInstallDelegate, uint32, bool);
DECLARE_MULTICAST_DELEGATE_TwoParams(FPlatformChunkInstallMultiDelegate, uint32, bool);

/** Delegate called when a Named Chunk either successfully installs or fails to install */
DECLARE_DELEGATE_OneParam(FPlatformNamedChunkCompleteDelegate, const FNamedChunkCompleteCallbackParam&);
DECLARE_MULTICAST_DELEGATE_OneParam(FPlatformNamedChunkCompleteMultiDelegate, const FNamedChunkCompleteCallbackParam&);

enum class ENamedChunkType : uint8
{
	Invalid,
	OnDemand,
	Language,
};

struct FChunkInstallationStatusDetail
{
	uint64 CurrentInstallSize;
	uint64 FullInstallSize;
	bool bIsInstalled;
};

/**
* Interface for platform specific chunk based install
**/
class IPlatformChunkInstall
{
public:

	/** Virtual destructor */
	virtual ~IPlatformChunkInstall() = default;

	/**
	 * Initialize the platform chunk installer in the background. May require opt-in for the platform-specific chunk installer due to backwards compatility concerns.
	 * @param OnInitialized Callback for when asyncronous initialization is completed.
	 */
	virtual void AsyncInit( TFunction<void(bool/*bSuccess*/)> OnInitComplete ) = 0;

	/** 
	 * Returns whether chunk installation is available (i.e. we are an installed packaged build etc)
	 */
	virtual bool IsAvailable() const = 0;

	/**
	 * Get the current location of a chunk with pakchunk index.
	 * @param PakchunkIndex	The id of the pak chunk.
	 * @return				Enum specifying whether the chunk is available to use, waiting to install, or does not exist.
	 **/
	virtual EChunkLocation::Type GetPakchunkLocation( int32 PakchunkIndex) = 0;

	/** 
	 * Check if a given reporting type is supported.
	 * @param ReportType	Enum specifying how progress is reported.
	 * @return				true if reporting type is supported on the current platform.
	 **/
	virtual bool GetProgressReportingTypeSupported(EChunkProgressReportingType::Type ReportType) = 0;		

	/**
	 * Get the current install progress of a chunk.  Let the user specify report type for platforms that support more than one.
	 * @param ChunkID		The id of the chunk to check.
	 * @param ReportType	The type of progress report you want.
	 * @return				A value whose meaning is dependent on the ReportType param.
	 **/
	virtual float GetChunkProgress( uint32 ChunkID, EChunkProgressReportingType::Type ReportType ) = 0;

	/**
	 * Inquire about the priority of chunk installation vs. game IO.
	 * @return				Paused, low or high priority.
	 **/
	virtual EChunkInstallSpeed::Type GetInstallSpeed() = 0;
	/**
	 * Specify the priority of chunk installation vs. game IO.
	 * @param InstallSpeed	Pause, low or high priority.
	 * @return				false if the operation is not allowed, otherwise true.
	 **/
	virtual bool SetInstallSpeed( EChunkInstallSpeed::Type InstallSpeed ) = 0;
	
	/**
	 * Hint to the installer that we would like to prioritize a specific chunk
	 * @param PakchunkIndex	The index of the pakchunk to prioritize.
	 * @param Priority		The priority for the chunk.
	 * @return				false if the operation is not allowed or the chunk doesn't exist, otherwise true.
	 **/
	virtual bool PrioritizePakchunk( int32 PakchunkIndex, EChunkPriority::Type Priority ) = 0;

	/**
	 * For platforms that support emulation of the Chunk install.  Starts transfer of the next chunk.
	 * Does nothing in a shipping build.
	 * @return				true if the operation succeeds.
	 **/
	virtual bool DebugStartNextChunk() = 0;

	/**
	 * Allow an external system to notify that a particular chunk ID has become available
	 * Initial use-case is for dynamically encrypted pak files to signal to the outside world that
	 * it has become available.
	 *
	 * @param InChunkID - ID of the chunk that has just become available
	 */
	virtual void ExternalNotifyChunkAvailable(uint32 InChunkID) = 0;

	/** 
	 * Request a delegate callback on chunk install completion or failure. Request may not be respected.
	 * @param Delegate		The delegate to call when any chunk is installed or fails to install
	 * @return				Handle to the bound delegate
	 */
	virtual FDelegateHandle AddChunkInstallDelegate( FPlatformChunkInstallDelegate Delegate ) = 0;

	/**
	 * Remove a delegate callback on chunk install completion.
	 * @param Delegate		The delegate to remove.
	 */
	virtual void RemoveChunkInstallDelegate( FDelegateHandle Delegate ) = 0;


	/**
	 * Check whether current platform supports chunk installation by name
	 * @return				whether Intelligent Install is supported
	 */
	virtual bool SupportsNamedChunkInstall() const = 0;

	/**
	 * Check whether the give chunk is being installed
	 * @param NamedChunk	The name of the chunk
	 * @return				whether installation task has been kicked
	 */
	virtual bool IsNamedChunkInProgress(const FName NamedChunk) = 0;

	/**
	 * Install the given named chunk
	 * @param NamedChunk	The name of the chunk
	 * @return				whether installation task has been kicked
	 **/
	virtual bool InstallNamedChunk(const FName NamedChunk) = 0;

	/**
	 * Uninstall the given named chunk
	 * @param NamedChunk	The name of the chunk
	 * @return				whether uninstallation task has been kicked
	 **/
	virtual bool UninstallNamedChunk(const FName NamedChunk) = 0;

	/**
	 * Install the given set of named chunks
	 * @param NamedChunks	The names of the chunks to install
	 * @return				whether installation task has been kicked
	 **/
	virtual bool InstallNamedChunks(const TArrayView<const FName>& NamedChunks) = 0;

	/**
	 * Uninstall the given set of named chunks
	 * @param NamedChunk	The names of the chunks to uninstall
	 * @return				whether uninstallation task has been kicked
	 **/
	virtual bool UninstallNamedChunks(const TArrayView<const FName>& NamedChunks) = 0;

	/**
	 * Get the current location of the given named chunk
	 * @param NamedChunk	The name of the chunk
	 * @return				Enum specifying whether the chunk is available to use, waiting to install, or does not exist.
	 **/
	virtual EChunkLocation::Type GetNamedChunkLocation(const FName NamedChunk) = 0;

	/**
	 * Get the current install progress of the given named chunk.  Let the user specify report type for platforms that support more than one.
	 * @param NamedChunk	The name of the chunk
	 * @param ReportType	The type of progress report you want.
	 * @return				A value whose meaning is dependent on the ReportType param.
	 **/
	virtual float GetNamedChunkProgress(const FName NamedChunk, EChunkProgressReportingType::Type ReportType) = 0;

	/**
	 * Hint to the installer that we would like to prioritize a specific chunk
	 * @param NamedChunk	The name of the chunk
	 * @param Priority		The priority for the chunk.
	 * @return				false if the operation is not allowed or the chunk doesn't exist, otherwise true.
	 **/
	virtual bool PrioritizeNamedChunk(const FName NamedChunk, EChunkPriority::Type Priority) = 0;

	/**
	 * Cancel an in-progress install of the given named chunk
	 * @param NamedChunk	The name of the chunk
	 * @return				whether the cancellation task has been kicked
	 **/
	virtual bool CancelNamedChunkInstall(const FName NamedChunk) = 0;

	/**
	 * Cancel an in-progress install of the given set of named chunks
	 * @param NamedChunks	The names of the chunks to cancel
	 * @return				whether the cancellation task has been kicked
	 **/
	virtual bool CancelNamedChunksInstall(const TArrayView<const FName>& NamedChunks) = 0;

	/**
	 * Query the type of the given named chunk
	 * @param NamedChunk	The name of the chunk
	 * @return				Enum indicating the type of chunk, if any
	 */
	virtual ENamedChunkType GetNamedChunkType(const FName NamedChunk) const = 0;

	/**
	 * Get a list of all the named chunks of the given type
	 * @param				Enum indicating the type of chunk
	 * @return				Array containing all named chunks of the given type
	 */
	virtual TArray<FName> GetNamedChunksByType(ENamedChunkType NamedChunkType) const = 0;

	/** 
	 * Request a delegate callback on named chunk install completion or failure. Request may not be respected.
	 * @param Delegate		The delegate to call when any named chunk is installed or fails to install
	 * @return				Handle to the bound delegate
	 */
	virtual FDelegateHandle AddNamedChunkCompleteDelegate( FPlatformNamedChunkCompleteDelegate Delegate ) = 0;

	/**
	 * Remove a delegate callback on named chunk install completion.
	 * @param Delegate		The delegate to remove.
	 */
	virtual void RemoveNamedChunkCompleteDelegate( FDelegateHandle Delegate ) = 0;



	/** 
	 * Returns whether this platform chunk installer implements all the API functions to support the platform chunk install bundle source
	 */
	virtual bool SupportsBundleSource() const = 0;

	/** 
	 * Set whether pak files are auto-mounted when they are installed (the default is that they are mounted)
	 * @param bEnabled	Whether to auto-mount pak files
	 * @return			false if this function is not supported
	 */
	virtual bool SetAutoPakMountingEnabled( bool bEnabled ) = 0;
	
	/**
	 * Get the list of pak files in the given named chunk.
	 * @param NamedChunk		The named chunk to query
	 * @param OutFilesInChunk	The pak files names in the named chunk
	 * @return					true if the named chunk is valid and this function is supported
	 */
	virtual bool GetPakFilesInNamedChunk( const FName NamedChunk, TArray<FString>& OutFilesInChunk) const = 0;

	/**
	 * Get detailed installation status for the given named chunk
	 * @param NamedChunk			The named chunk to query
	 * @param OutChunkStatusDetail	(out) structure that will contain the status detail
	 * @return						true if the structure has been filled in
	 */
	virtual bool GetNamedChunkInstallationStatus( const FName NamedChunk, FChunkInstallationStatusDetail& OutChunkStatusDetail ) const = 0;

	/**
	 * Determine if the given named chunk is suitable for the current system locale
	 * @param NamedChunk			The named chunk to query
	 * @returns						false if the chunk is associated with a different locale
	 */
	virtual bool IsNamedChunkForCurrentLocale( const FName NamedChunk ) const = 0;
	

protected:
		/**
		 * Get the current location of a chunk.
		 * Pakchunk index and platform chunk id are not always the same.  Call GetPakchunkLocation instead of calling from outside.
		 * @param ChunkID		The id of the chunk to check.
		 * @return				Enum specifying whether the chunk is available to use, waiting to install, or does not exist.
		 **/
		virtual EChunkLocation::Type GetChunkLocation(uint32 ChunkID) = 0;

		/**
		 * Hint to the installer that we would like to prioritize a specific chunk
		 * @param ChunkID		The id of the chunk to prioritize.
		 * @param Priority		The priority for the chunk.
		 * @return				false if the operation is not allowed or the chunk doesn't exist, otherwise true.
		 **/
		virtual bool PrioritizeChunk(uint32 ChunkID, EChunkPriority::Type Priority) = 0;

};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
/**
 * Generic implementation of chunk based install
 */
class FGenericPlatformChunkInstall : public IPlatformChunkInstall
{
public:
	CORE_API virtual void AsyncInit( TFunction<void(bool/*bSuccess*/)> OnInitComplete ) override;

	virtual bool IsAvailable() const
	{
		return true;
	}

	virtual EChunkLocation::Type GetPakchunkLocation( int32 PakchunkIndex ) override
	{
		return GetChunkLocation(PakchunkIndex);
	}

	virtual bool PrioritizePakchunk(int32 PakchunkIndex, EChunkPriority::Type Priority)
	{
		return PrioritizeChunk(PakchunkIndex, Priority);
	}

	virtual bool GetProgressReportingTypeSupported(EChunkProgressReportingType::Type ReportType) override
	{
		if (ReportType == EChunkProgressReportingType::PercentageComplete)
		{
			return true;
		}

		return false;
	}

	virtual float GetChunkProgress( uint32 ChunkID, EChunkProgressReportingType::Type ReportType ) override
	{
		if (ReportType == EChunkProgressReportingType::PercentageComplete)
		{
			return 100.0f;
		}		
		return 0.0f;
	}

	virtual EChunkInstallSpeed::Type GetInstallSpeed() override
	{
		return EChunkInstallSpeed::Paused;
	}

	virtual bool SetInstallSpeed( EChunkInstallSpeed::Type InstallSpeed ) override
	{
		return false;
	}
	
	virtual bool PrioritizeChunk( uint32 ChunkID, EChunkPriority::Type Priority ) override
	{
		return false;
	}

	virtual bool DebugStartNextChunk() override
	{
		return true;
	}

	virtual void ExternalNotifyChunkAvailable(uint32 InChunkID) override
	{
		InstallDelegate.Broadcast(InChunkID, true);
	}

	virtual FDelegateHandle AddChunkInstallDelegate(FPlatformChunkInstallDelegate Delegate) override
	{
		return InstallDelegate.Add(Delegate);
	}

	virtual void RemoveChunkInstallDelegate(FDelegateHandle Delegate) override
	{
		InstallDelegate.Remove(Delegate);
	}

	virtual bool SupportsNamedChunkInstall() const override
	{
		return false;
	}

	virtual bool IsNamedChunkInProgress(const FName NamedChunk) override
	{
		return false;
	}

	virtual bool InstallNamedChunk(const FName NamedChunk) override
	{
		return InstallNamedChunks(MakeArrayView(&NamedChunk,1));
	}

	virtual bool UninstallNamedChunk(const FName NamedChunk) override
	{
		return UninstallNamedChunks(MakeArrayView(&NamedChunk,1));
	}

	virtual bool InstallNamedChunks(const TArrayView<const FName>& NamedChunks) override
	{
		return false;
	}

	virtual bool UninstallNamedChunks(const TArrayView<const FName>& NamedChunks) override
	{
		return false;
	}

	virtual bool CancelNamedChunkInstall(const FName NamedChunk) override
	{
		return CancelNamedChunksInstall(MakeArrayView(&NamedChunk, 1));
	}

	virtual bool CancelNamedChunksInstall(const TArrayView<const FName>& NamedChunks) override
	{
		return false;
	}

	virtual EChunkLocation::Type GetNamedChunkLocation(const FName NamedChunk) override
	{
		return EChunkLocation::NotAvailable;
	}

	virtual float GetNamedChunkProgress(const FName NamedChunk, EChunkProgressReportingType::Type ReportType) override
	{
		return 0.0f;
	}

	virtual bool PrioritizeNamedChunk(const FName NamedChunk, EChunkPriority::Type Priority) override
	{
		return false;
	}

	virtual ENamedChunkType GetNamedChunkType(const FName NamedChunk) const override
	{
		return ENamedChunkType::Invalid;
	}

	virtual TArray<FName> GetNamedChunksByType(ENamedChunkType NamedChunkType) const override
	{
		return TArray<FName>();
	}

	virtual void RemoveNamedChunkCompleteDelegate(FDelegateHandle Delegate) override
	{
		NamedChunkCompleteDelegate.Remove(Delegate);
	}

	virtual FDelegateHandle AddNamedChunkCompleteDelegate(FPlatformNamedChunkCompleteDelegate Delegate) override
	{
		return NamedChunkCompleteDelegate.Add(Delegate);
	}


	virtual bool SupportsBundleSource() const override 
	{ 
		return false; 
	}

	virtual bool SetAutoPakMountingEnabled( bool bEnabled ) 
	{ 
		return false; 
	}

	virtual bool GetPakFilesInNamedChunk( const FName NamedChunk, TArray<FString>& OutFilesInChunk) const override 
	{ 
		return false; 
	}

	virtual bool GetNamedChunkInstallationStatus( const FName NamedChunk, FChunkInstallationStatusDetail& OutChunkStatusDetail ) const override 
	{ 
		return false; 
	}

	virtual bool IsNamedChunkForCurrentLocale( const FName NamedChunk ) const 
	{ 
		return true; 
	}

protected:

	CORE_API void DoNamedChunkCompleteCallbacks( const FName NamedChunk, EChunkLocation::Type Location, bool bHasSucceeded ) const;
	CORE_API void DoNamedChunkCompleteCallbacks( const TArrayView<const FName>& NamedChunks, EChunkLocation::Type Location, bool bHasSucceeded ) const;

	/** Delegates called when installation succeeds or fails */
	FPlatformChunkInstallMultiDelegate InstallDelegate;
	FPlatformNamedChunkCompleteMultiDelegate NamedChunkCompleteDelegate;

	virtual EChunkLocation::Type GetChunkLocation(uint32 ChunkID) override
	{
		return EChunkLocation::LocalFast;
	}
};



PRAGMA_ENABLE_DEPRECATION_WARNINGS
