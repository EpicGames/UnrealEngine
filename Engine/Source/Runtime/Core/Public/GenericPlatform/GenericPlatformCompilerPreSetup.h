// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Cause cirular dependencies for header units

#ifndef UE_CLANG_SUPPRESS
	#define UE_CLANG_SUPPRESS
#endif

#ifndef PRAGMA_DISABLE_DEPRECATION_WARNINGS
	#define PRAGMA_DISABLE_DEPRECATION_WARNINGS
#endif

#ifndef PRAGMA_ENABLE_DEPRECATION_WARNINGS
	#define PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

#ifndef EMIT_CUSTOM_WARNING_AT_LINE
	#define EMIT_CUSTOM_WARNING_AT_LINE(Line, Warning)
#endif // EMIT_CUSTOM_WARNING_AT_LINE

#ifndef EMIT_CUSTOM_ERROR_AT_LINE
	#define EMIT_CUSTOM_ERROR_AT_LINE(Line, Error)
#endif // EMIT_CUSTOM_ERROR_AT_LINE

#ifndef EMIT_CUSTOM_WARNING
	#define EMIT_CUSTOM_WARNING(Warning) \
		EMIT_CUSTOM_WARNING_AT_LINE(__LINE__, Warning)
#endif // EMIT_CUSTOM_WARNING

#ifndef EMIT_CUSTOM_ERROR
	#define EMIT_CUSTOM_ERROR(Error) \
		EMIT_CUSTOM_ERROR_AT_LINE(__LINE__, Error)
#endif // EMIT_CUSTOM_ERROR

#ifndef UE_DEPRECATED_MACRO
	#if UE_WARNINGS_AS_ERRORS
		#define UE_DEPRECATED_MACRO(Version, Message) EMIT_CUSTOM_ERROR(Message " - Please update your code to the new API before upgrading to the next release, otherwise your project will no longer compile.")
	#else
		#define UE_DEPRECATED_MACRO(Version, Message) EMIT_CUSTOM_WARNING(Message " - Please update your code to the new API before upgrading to the next release, otherwise your project will no longer compile.")
	#endif
#endif

#ifndef PRAGMA_DEFAULT_VISIBILITY_START
	#define PRAGMA_DEFAULT_VISIBILITY_START
#endif

#ifndef PRAGMA_DEFAULT_VISIBILITY_END
	#define PRAGMA_DEFAULT_VISIBILITY_END
#endif

#ifndef PRAGMA_DISABLE_BUFFER_OVERRUN_WARNING
	#define PRAGMA_DISABLE_BUFFER_OVERRUN_WARNING
#endif

#ifndef PRAGMA_ENABLE_BUFFER_OVERRUN_WARNING
	#define PRAGMA_ENABLE_BUFFER_OVERRUN_WARNING
#endif

#ifndef UE_UNREACHABLE_CODE
	#define UE_UNREACHABLE_CODE(...) \
		PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS \
		__VA_ARGS__ \
		PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS
#endif

// Disable CA warnings around SDK includes - default to those disabled at the compiler level
#ifndef THIRD_PARTY_INCLUDES_START
	#define THIRD_PARTY_INCLUDES_START UE_COMPILER_THIRD_PARTY_INCLUDES_START
#endif

#ifndef THIRD_PARTY_INCLUDES_END
	#define THIRD_PARTY_INCLUDES_END UE_COMPILER_THIRD_PARTY_INCLUDES_END
#endif

#ifndef PGO_LINK_DISABLE_WARNINGS
	#define PGO_LINK_DISABLE_WARNINGS
#endif

#ifndef PGO_LINK_ENABLE_WARNINGS
	#define PGO_LINK_ENABLE_WARNINGS
#endif
