// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformStackWalk.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "Misc/SecureHash.h"
#include "Misc/Timespan.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

struct FDateTime;
struct FGuid;
struct FScopedAdditionalCrashContextProvider;

#ifndef WITH_ADDITIONAL_CRASH_CONTEXTS
#define WITH_ADDITIONAL_CRASH_CONTEXTS 1
#endif

struct FProgramCounterSymbolInfo;

/** Defines special exit codes used to diagnose abnormal terminations. The code values are arbitrary, but easily recongnizable in decimal. They are meant to be
    used with the out-of-process monitoring/analytics in order to figure out unexpected cases. */
enum ECrashExitCodes : int32
{
	/** Used by out-of-process monitor in analytics report, the application is still running, but out-of-process monitor was requested to exit before the application exit code could be read. */
	MonitoredApplicationStillRunning = 777001,

	/** Used by out-of-process monitor in analytics report, the application is not running anymore, but the out-of-process monitor could not read the Editor exit code (either is is not supported by the OS or is not available). */
	MonitoredApplicationExitCodeNotAvailable = 777002,

	/** Used by the application when the crash reporter crashed itself while reporting a crash.*/
	CrashReporterCrashed = 777003,

	/** Used by the application when the crash handler crashed itself (crash in the __except() clause for example).*/
	CrashHandlerCrashed = 777004,

	/** Used by the application to flag when it detects that its out-of-process application supposed to report the bugs died (ex if the Editor detects that CrashReportClientEditor is not running anymore as expected).*/
	OutOfProcessReporterExitedUnexpectedly = 777005,

	/** Application crashed during static initialization. It may or may not have been able to have sent a crash report. */
	CrashDuringStaticInit = 777006,

	/** Used as MonitorExceptCode in analytics to track how often the out-of-process CRC exits because of a failed check. */
	OutOfProcessReporterCheckFailed = 777007,

	/** The exception code used for ensure, in case a kernel driver callback happens at in a dispatch level where SEH (on windows) is disabled. */
	UnhandledEnsure = 777008,
};

/** Enumerates crash description versions. */
enum class ECrashDescVersions : int32
{
	/** Introduces a new crash description format. */
	VER_1_NewCrashFormat,

	/** Added misc properties (CPU,GPU,OS,etc), memory related stats and platform specific properties as generic payload. */
	VER_2_AddedNewProperties,

	/** Using crash context when available. */
	VER_3_CrashContext = 3,
};

/** Enumerates crash dump modes. */
enum class ECrashDumpMode : int32
{
	/** Default minidump settings. */
	Default = 0,

	/** Full memory crash minidump */
	FullDump = 1,

	/** Full memory crash minidump, even on ensures */
	FullDumpAlways = 2,
};

/** Portable stack frame */
struct FCrashStackFrame
{
	FString ModuleName;
	uint64 BaseAddress;
	uint64 Offset;

	FCrashStackFrame(FString ModuleNameIn, uint64 BaseAddressIn, uint64 OffsetIn)
		: ModuleName(MoveTemp(ModuleNameIn))
		, BaseAddress(BaseAddressIn)
		, Offset(OffsetIn)
	{
	}
};

/** Portable thread stack frame */
struct FThreadStackFrames {
	FString						ThreadName;
	uint32						ThreadId;
	TArray<FCrashStackFrame>	StackFrames;
};

enum class ECrashContextType
{
	Crash,
	Assert,
	Ensure,
	Stall,
	GPUCrash,
	Hang,
	OutOfMemory,
	AbnormalShutdown,
	VerseRuntimeError,

	Max
};

/** In development mode we can cause crashes in order to test reporting systems. */
enum class ECrashTrigger
{
	Debug = -1,
	Normal = 0
};

/**
 * Tristate to identify a session which is attended or unattended (ie. usually automated testing)
 * Determination requires command line arguments - therefore if not available, status is unknown 
 */
enum class EUnattendedStatus : uint8
{
	Unknown,
	Attended,
	Unattended
};

CORE_API const TCHAR* AttendedStatusToString(const EUnattendedStatus Status);

#define CR_MAX_ERROR_MESSAGE_CHARS 2048
#define CR_MAX_DIRECTORY_CHARS 256
#define CR_MAX_SYMBOL_CHARS 128
#define CR_MAX_STACK_FRAMES 256
#define CR_MAX_THREAD_NAME_CHARS 64
#define CR_MAX_THREADS 512
#define CR_MAX_GENERIC_FIELD_CHARS 64
#define CR_MAX_COMMANDLINE_CHARS 1024
#define CR_MAX_RICHTEXT_FIELD_CHARS 512
#define CR_MAX_DYNAMIC_BUFFER_CHARS 1024*32
#define CR_MAX_GPU_BREADCRUMBS_QUEUES 2
#define CR_MAX_GPU_BREADCRUMBS_STRING_CHARS 1024*8

/**
 * Fixed size structure that holds session specific state.
 */
struct FSessionContext 
{
	bool 					bIsInternalBuild;
	bool 					bIsPerforceBuild;
	bool 					bWithDebugInfo;
	bool 					bIsSourceDistribution;
	bool 					bIsUERelease;
	bool					bIsOOM;
	bool					bIsExitRequested;
	bool					bIsStuck;
	uint32					ProcessId;
	int32 					LanguageLCID;
	int32 					NumberOfCores;
	int32 					NumberOfCoresIncludingHyperthreads;
	uint32					CPUModelID;
	uint32					CPUMicrocodeRevision;
	int32 					SecondsSinceStart;
	int32 					CrashDumpMode;
	int32					CrashTrigger;
	uint32					StuckThreadId;
	int32					OOMAllocationAlignment;
	uint64					OOMAllocationSize;
	TCHAR 					EngineVersion[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					EngineCompatibleVersion[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					BuildVersion[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					GameName[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR					EngineMode[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR					EngineModeEx[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					ExecutableName[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR					BuildConfigurationName[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					DeploymentName[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					BaseDir[CR_MAX_DIRECTORY_CHARS];
	TCHAR 					RootDir[CR_MAX_DIRECTORY_CHARS];
	TCHAR 					EpicAccountId[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					LoginIdStr[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR					SymbolsLabel[CR_MAX_SYMBOL_CHARS];
	TCHAR 					OsVersion[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					OsSubVersion[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					CPUVendor[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					CPUBrand[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					PrimaryGPUBrand[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					UserName[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					DefaultLocale[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					CrashGUIDRoot[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					UserActivityHint[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					GameSessionID[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					CommandLine[CR_MAX_COMMANDLINE_CHARS];
	TCHAR 					CrashReportClientRichText[CR_MAX_RICHTEXT_FIELD_CHARS];
	TCHAR 					GameStateName[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR 					CrashConfigFilePath[CR_MAX_DIRECTORY_CHARS];
	TCHAR					AttendedStatus[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR					PlatformName[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR					PlatformNameIni[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR					AnticheatProvider[CR_MAX_GENERIC_FIELD_CHARS];
	FPlatformMemoryStats	MemoryStats;
};

/** Additional user settings to be communicated to crash reporting client. */
struct FUserSettingsContext
{
	bool					bNoDialog = false;
	bool					bSendUnattendedBugReports = false;
	bool					bSendUsageData = false;
	bool					bImplicitSend = false;
	TCHAR					LogFilePath[CR_MAX_DIRECTORY_CHARS];
};

/** Fixed size structure holding GPU breadcrumbs information, to be communicated to the crash reporting client. */
struct FGPUBreadcrumbsSharedContext
{
	struct FQueueData {
		TCHAR QueueName[CR_MAX_GENERIC_FIELD_CHARS];
		TCHAR FullHash[CR_MAX_GENERIC_FIELD_CHARS];
		TCHAR ActiveHash[CR_MAX_GENERIC_FIELD_CHARS];
		TCHAR Breadcrumbs[CR_MAX_GPU_BREADCRUMBS_STRING_CHARS];
	};

	TCHAR SourceName[CR_MAX_GENERIC_FIELD_CHARS];
	TCHAR Version[CR_MAX_GENERIC_FIELD_CHARS];

	uint32 NumQueues = 0;
	FQueueData Queues[CR_MAX_GPU_BREADCRUMBS_QUEUES];
};

/**
 * Fixed size struct holds crash information and session specific state. It is designed
 * to shared between processes (e.g. Game and CrashReporterClient).
 */
struct FSharedCrashContext
{
	// Exception info
	TCHAR					ErrorMessage[CR_MAX_ERROR_MESSAGE_CHARS];
	uint32					ThreadIds[CR_MAX_THREADS];
	TCHAR					ThreadNames[CR_MAX_THREAD_NAME_CHARS * CR_MAX_THREADS];
	uint32					NumThreads;
	uint32					CrashingThreadId;
	ECrashContextType		CrashType;

	// Additional user settings.
	FUserSettingsContext	UserSettings;

	// Platform specific crash context (must be portable)
	void*					PlatformCrashContext;
	// Directory for dumped files
	TCHAR					CrashFilesDirectory[CR_MAX_DIRECTORY_CHARS];
	// Game/Engine information not possible to catch out of process
	FSessionContext			SessionContext;
	// Count and offset into dynamic buffer to comma separated plugin list
	uint32					EnabledPluginsNum;
	uint32					EnabledPluginsOffset;
	// Count and offset into dynamic buffer to comma separated key=value data for engine information
	uint32					EngineDataNum;
	uint32					EngineDataOffset;
	// Count and offset into dynamic buffer to comma separated key=value data for  game information
	uint32					GameDataNum;
	uint32					GameDataOffset;
	// Count and offset into dynamic buffer to comma separated file list
	uint32					FilePathsNum;
	uint32					FilePathsOffset;
	// Fixed size dynamic buffer
	TCHAR					DynamicData[CR_MAX_DYNAMIC_BUFFER_CHARS];

	// Program counter address where the error occurred.
	void*					ErrorProgramCounter;

	// Instruction address where the exception was raised that initiated crash reporting
	void*					ExceptionProgramCounter;

	// GPU breadcrumbs.
	FGPUBreadcrumbsSharedContext  GPUBreadcrumbs;
};

#if WITH_ADDITIONAL_CRASH_CONTEXTS

/**
 * Interface for callbacks to add context to the crash report.
 */
struct FCrashContextExtendedWriter
{
	virtual ~FCrashContextExtendedWriter() = default;

	/** Adds a named buffer to the report. Intended for larger payloads. */
	virtual void AddBuffer(const TCHAR* Identifier, const uint8* Data, uint32 DataSize) = 0;

	/** Add a named buffer containing a string to the report. */
	virtual void AddString(const TCHAR* Identifier, const TCHAR* DataStr) = 0;

	/** MoveFile a file into the report
	* FromFilePath - the full path of the file you wish to move into the report.
	* The base filename is used for the destination file.
	*/
	virtual bool MoveFile(const TCHAR* FromFilePath) = 0;
};

/** Simple Delegate for additional crash context. */
DECLARE_MULTICAST_DELEGATE_OneParam(FAdditionalCrashContextDelegate, FCrashContextExtendedWriter&);

/** Informations the callback might need to gather additional crash informations */
struct FAdditionalCrashContextParams
{
	uint32 CrashedThreadId = ~(uint32)0;
};

DECLARE_MULTICAST_DELEGATE_TwoParams(FAdditionalCrashContextDelegateEx, FCrashContextExtendedWriter&, const FAdditionalCrashContextParams& Params);

#endif //WITH_ADDITIONAL_CRASH_CONTEXTS

/** Delegates for engine and game data set / reset */
DECLARE_MULTICAST_DELEGATE(FEngineDataResetDelegate);
DECLARE_MULTICAST_DELEGATE_TwoParams(FEngineDataSetDelegate, const FString&, const FString&);

DECLARE_MULTICAST_DELEGATE(FGameDataResetDelegate);
DECLARE_MULTICAST_DELEGATE_TwoParams(FGameDataSetDelegate, const FString&, const FString&);

struct FThreadCallStack
{
	TConstArrayView<uint64> StackFrames;
	const TCHAR* ThreadName;
	uint32 ThreadId;
};

/** GPU breadcrumbs. */
struct FGPUBreadcrumbCrashData
{
	/**
	 * This must be changed whenever the format of the breadcrumb string
	 * changes, in order to help parsers in dealing with strings from multiple
	 * versions.
	 */
	static constexpr TCHAR const CurrentVersion[] = TEXT("526BDA74-7A81-44C3-B0FD-9DBF80973C25");

	enum class EState : uint8
	{
		NotStarted = 0,
		Active     = 1,
		Finished   = 2
	};

	// These are serialized. Do not change this array without bumping the CurrentVersion.
	TCHAR const static constexpr StateChars[] =
	{
		TEXT('N'),
		TEXT('A'),
		TEXT('F'),
	};

	struct FQueueData
	{
		FString BreadcrumbString;
		FSHAHash FullHash;
		FSHAHash ActiveHash;

		operator bool() const { return !BreadcrumbString.IsEmpty(); }
	};
	TMap<FString, FQueueData> Queues;
	FString SourceName;
	FString Version;

	class FSerializer
	{
		FString String;
		FSHA1 FullHash, ActiveHash;
		TArray<bool> ChildStack;

		// Sanitize the event name string to remove characters that are used as delimiters for parsing.
		static FString Sanitize(FString const& Name);

		// Event names include parameters, mostly numeric (e.g. "Frame 1234"), that should be ignored when computing the hash.
		static FString SanitizeForHash(FString const& Name);

	public:
		CORE_API void BeginNode(FString const& Name, EState State);
		CORE_API void EndNode();

		CORE_API FQueueData GetResult();
	};

	FGPUBreadcrumbCrashData(TCHAR const* InSourceName, TCHAR const* InVersion = CurrentVersion)
		: SourceName(InSourceName), Version(InVersion)
	{}
};

/**
 *	Contains a runtime crash's properties that are common for all platforms.
 *	This may change in the future.
 */
struct FGenericCrashContext
{
public:

	CORE_API static const ANSICHAR* const CrashContextRuntimeXMLNameA;
	CORE_API static const TCHAR* const CrashContextRuntimeXMLNameW;

	CORE_API static const ANSICHAR* const CrashConfigFileNameA;
	CORE_API static const TCHAR* const CrashConfigFileNameW;
	CORE_API static const TCHAR* const CrashConfigExtension;
	CORE_API static const TCHAR* const ConfigSectionName;
	CORE_API static const TCHAR* const CrashConfigPurgeDays;
	CORE_API static const TCHAR* const CrashGUIDRootPrefix;

	CORE_API static const TCHAR* const CrashReportOptionalAttachmentsDirName;

	CORE_API static const TCHAR* const CrashContextExtension;
	CORE_API static const TCHAR* const RuntimePropertiesTag;
	CORE_API static const TCHAR* const DeploymentNameTag;
	CORE_API static const TCHAR* const PlatformPropertiesTag;
	CORE_API static const TCHAR* const EngineDataTag;
	CORE_API static const TCHAR* const GameDataTag;
	CORE_API static const TCHAR* const RequestDataTag;
	CORE_API static const TCHAR* const GameNameTag;
	CORE_API static const TCHAR* const EnabledPluginsTag;
	CORE_API static const TCHAR* const UEMinidumpName;
	CORE_API static const TCHAR* const NewLineTag;
	CORE_API static const TCHAR* const CrashVersionTag;
	CORE_API static const TCHAR* const ExecutionGuidTag;
	CORE_API static const TCHAR* const CrashGuidTag;
	CORE_API static const TCHAR* const IsEnsureTag;
	CORE_API static const TCHAR* const IsStallTag;
	CORE_API static const TCHAR* const IsAssertTag;
	CORE_API static const TCHAR* const CrashTypeTag;
	CORE_API static const TCHAR* const ErrorMessageTag;
	CORE_API static const TCHAR* const CrashReporterMessageTag;
	CORE_API static const TCHAR* const AttendedStatusTag;
	CORE_API static const TCHAR* const SecondsSinceStartTag;
	CORE_API static const TCHAR* const BuildVersionTag;
	CORE_API static const TCHAR* const CallStackTag;
	CORE_API static const TCHAR* const PortableCallStackTag;
	CORE_API static const TCHAR* const PortableCallStackHashTag;
	CORE_API static const TCHAR* const IsRequestingExitTag;
	CORE_API static const TCHAR* const LogFilePathTag;
	CORE_API static const TCHAR* const ProcessIdTag;
	CORE_API static const TCHAR* const IsInternalBuildTag;
	CORE_API static const TCHAR* const IsPerforceBuildTag;
	CORE_API static const TCHAR* const IsWithDebugInfoTag;
	CORE_API static const TCHAR* const IsSourceDistributionTag;

	static constexpr inline int32 CrashGUIDLength = 128;

	CORE_API static const TCHAR* const CrashTypeCrash;
	CORE_API static const TCHAR* const CrashTypeAssert;
	CORE_API static const TCHAR* const CrashTypeEnsure;
	CORE_API static const TCHAR* const CrashTypeStall;
	CORE_API static const TCHAR* const CrashTypeGPU;
	CORE_API static const TCHAR* const CrashTypeHang;
	CORE_API static const TCHAR* const CrashTypeAbnormalShutdown;
	CORE_API static const TCHAR* const CrashTypeOutOfMemory;
	CORE_API static const TCHAR* const CrashTypeVerseRuntimeError;

	CORE_API static const TCHAR* const EngineModeExUnknown;
	CORE_API static const TCHAR* const EngineModeExDirty;
	CORE_API static const TCHAR* const EngineModeExVanilla;

	// A guid that identifies this particular execution. Allows multiple crash reports from the same run of the project to be tied together
	CORE_API static const FGuid ExecutionGuid;

	/** Initializes crash context related platform specific data that can be impossible to obtain after a crash. */
	CORE_API static void Initialize();

	/** Initialized crash context, using a crash context (e.g. shared from another process). */
	CORE_API static void InitializeFromContext(const FSessionContext& Context, const TCHAR* EnabledPlugins, const TCHAR* EngineData, const TCHAR* GameData, const TCHAR* FilePathData, const FGPUBreadcrumbsSharedContext* GPUBreadcrumbs);

	/** Get the current cached session context */
	CORE_API static const FSessionContext& GetCachedSessionContext();

	/** Gets the current standardized game name for use in a Crash Reporter report. */
	CORE_API static FString GetGameName();

	/**
	 * @return true, if the generic crash context has been initialized.
	 */
	static bool IsInitalized()
	{
		return bIsInitialized;
	}

	/**
	 * @return true if walking the crashed call stack and writing the minidump is being handled out-of-process.
	 * @note The reporting itself (showing the crash UI and sending the report is always done out of process)
	 */
	static bool IsOutOfProcessCrashReporter()
	{
		return OutOfProcessCrashReporterPid != 0;
	}

	/**
	 * @return a non-zero value if crash reporter process is used to monitor the session, capture the call stack and write the minidump, otherwise, this is done inside the crashing process.
	 */
	static uint32 GetOutOfProcessCrashReporterProcessId()
	{
		return OutOfProcessCrashReporterPid;
	}

	/**
	 * Set whether or not the out-of-process crash reporter is running. A non-zero process id means that crash artifacts like the call stack and then minidump are
	 * built in a separated background process. The reporting itself, i.e. packaging and sending the crash artifacts is always done out of process.
	 * @note CrashReportClient (CrashReportClientEditor for the Editor) can be configured to wait for crash, capture the crashed process callstack, write the minidump, collect all crash artifacts
	 *       and send them (out-of-process reporting) or just collect and send them (in-process reporting because the crashing process creates all crash artifacts itself).
	 */
	static void SetOutOfProcessCrashReporterPid(uint32 ProcessId)
	{
		OutOfProcessCrashReporterPid = ProcessId;
	}

	/**
	 * Set the out of process crash reporter exit code if known. The out of process reporter is expected to run in background, waiting for a signal to handle a
	 * crashes/ensures/assert, but sometimes it crashes. If the engine detects that its associated out of process crash reporter died and if the child process exit
	 * code can be retrieved, it can be exposed through this function.
	 * @see GetOutOfProcessCrashReporterExitCode
	 */
	CORE_API static void SetOutOfProcessCrashReporterExitCode(int32 ExitCode);

	/**
	 * Return the out-of-process crash reporter exit code if available. The exit code is available if crash reporter process died while the application it monitors was still running.
	 * Then engine periodically poll the health of the crash reporter process and try to read its exit code if it unexpectedly died.
	 * @note This function is useful to try diagnose why the crash reporter died (crashed/killed/asserted) and gather data for the analytics.
	 */
	CORE_API static TOptional<int32> GetOutOfProcessCrashReporterExitCode();

	/** Default constructor. Optionally pass a process handle if building a crash context for a process other then current. */
	CORE_API FGenericCrashContext(ECrashContextType InType, const TCHAR* ErrorMessage);

	virtual ~FGenericCrashContext() = default;

	/** Get the file path to the temporary session context file that we create for the given process. */
	CORE_API static FString GetTempSessionContextFilePath(uint64 ProcessID);

	/** Clean up expired context files that were left-over on the user disks (because the consumer crashed and/or failed to delete it). */
	CORE_API static void CleanupTempSessionContextFiles(const FTimespan& ExpirationAge);

	/** Serializes all data to the buffer. */
	CORE_API void SerializeContentToBuffer() const;

	/**
	 * @return the buffer containing serialized data.
	 */
	const FString& GetBuffer() const
	{
		return CommonBuffer;
	}

	/**
	 * @return a globally unique crash name.
	 */
	CORE_API void GetUniqueCrashName(TCHAR* GUIDBuffer, int32 BufferSize) const;

	/**
	 * @return whether this crash is a full memory minidump
	 */
	CORE_API const bool IsFullCrashDump() const;

	/** Serializes crash's informations to the specified filename. Should be overridden for platforms where using FFileHelper is not safe, all POSIX platforms. */
	CORE_API virtual void SerializeAsXML( const TCHAR* Filename ) const;

	/** 
	 * Serializes session context to the given buffer. 
	 * NOTE: Assumes that the buffer already has a header and section open.
	 */
	CORE_API static void SerializeSessionContext(FString& Buffer);
	
	template <typename Type>
	void AddCrashProperty(const TCHAR* PropertyName, const Type& Value) const
	{
		AddCrashPropertyInternal(CommonBuffer, PropertyName, Value);
	}

	template <typename Type>
	static void AddCrashProperty(FString& Buffer, const TCHAR* PropertyName, const Type& Value)
	{
		AddCrashPropertyInternal(Buffer, PropertyName, Value);
	}

	/** Escapes and appends specified text to XML string */
	CORE_API static void AppendEscapedXMLString(FString& OutBuffer, FStringView Text );
	CORE_API static void AppendEscapedXMLString(FStringBuilderBase& OutBuffer, FStringView Text);

	CORE_API static void AppendPortableCallstack(FString& OutBuffer, TConstArrayView<FCrashStackFrame> StackFrames);

	/** Unescapes a specified XML string, naive implementation. */
	CORE_API static FString UnescapeXMLString( const FString& Text );

	/** Helper to get the standard string for the crash type based on crash event bool values. */
	CORE_API static const TCHAR* GetCrashTypeString(ECrashContextType Type);

	/** Get the Game Name of the crash */
	CORE_API static FString GetCrashGameName();

	/** Helper to get the crash report client config filepath saved by this instance and copied to each crash report folder. */
	CORE_API static const TCHAR* GetCrashConfigFilePath();

	/** Helper to get the crash report client config folder used by GetCrashConfigFilePath(). */
	CORE_API static const TCHAR* GetCrashConfigFolder();

	/** Helper to clean out old files in the crash report client config folder. */
	CORE_API static void PurgeOldCrashConfig();

	/** Set or change the epic account id associated with the crash session. Will override the epic account id stored in the registry for reporting when present. */ 
	CORE_API static void SetEpicAccountId(const FString& EpicAccountId);

	/** Clears the engine data dictionary */
	CORE_API static void ResetEngineData();

	/** Accessor for engine data reset callback delegate */
	static FEngineDataResetDelegate& OnEngineDataResetDelegate() { return OnEngineDataReset; }

	/** Updates (or adds if not already present) arbitrary engine data to the crash context (will remove the key if passed an empty string) */
	CORE_API static void SetEngineData(const FString& Key, const FString& Value);

	/** Updates (or adds if not already present) GPU breadcrumb data. */
	CORE_API static void SetGPUBreadcrumbs(FGPUBreadcrumbCrashData&& Data);

	/** Accessor for engine data change callback delegate */
	static FEngineDataSetDelegate& OnEngineDataSetDelegate() { return OnEngineDataSet; }

	/** Get the engine data dictionary */
	CORE_API static const TMap<FString, FString>& GetEngineData();

	/** Clears the game data dictionary */
	CORE_API static void ResetGameData();

	/** Accessor for game data reset callback delegate */
	static FGameDataResetDelegate& OnGameDataResetDelegate() { return OnGameDataReset; }

	/** Updates (or adds if not already present) arbitrary game data to the crash context (will remove the key if passed an empty string) */
	CORE_API static void SetGameData(const FString& Key, const FString& Value);

	/** Accessor for game data change callback delegate */
	static FGameDataSetDelegate& OnGameDataSetDelegate() { return OnGameDataSet; }

	/** Get the game data dictionary */
	CORE_API static const TMap<FString, FString>& GetGameData();

	/** Adds a plugin descriptor string to the enabled plugins list in the crash context */
	CORE_API static void AddPlugin(const FString& PluginDesc);

	/** Adds a file to bundle with the crash report */
	CORE_API static void AddFile(const FString& AbsoluteSourcePath);

	/** Flushes the logs. In the case of in memory logs is used on this configuration, dumps them to file. Returns the name of the file */
	CORE_API static FString DumpLog(const FString& CrashFolderAbsolute);

	/** Collects additional crash context providers. See FAdditionalCrashContextStack. */
	UE_DEPRECATED(5.8, "Use the overload with FAdditionalCrashContextParams")
	CORE_API static void DumpAdditionalContext(const TCHAR* CrashFolderAbsolute);

	/** Collects additional crash context providers. See FAdditionalCrashContextStack. */
	CORE_API static void DumpAdditionalContext(const TCHAR* CrashFolderAbsolute, const FAdditionalCrashContextParams& Params);

	/** Initializes a shared crash context from current state. Will not set all fields in Dst. */
	CORE_API static void CopySharedCrashContext(FSharedCrashContext& Dst);

	/** We can't gather memory stats in crash handling function, so we gather them just before raising
	  * exception and use in crash reporting. 
	  */
	CORE_API static void SetMemoryStats(const FPlatformMemoryStats& MemoryStats);

	/** Sets the Anticheat client provider. */
	CORE_API static void SetAnticheatProvider(const FString& AnticheatProvider);

	/** Sets a flag that one of the threads is stuck.
	 *  This is meant to be bound to the ThreadHeartBeat::OnThreadStuck delegate. Not all platforms register to save this flag. */
	CORE_API static void OnThreadStuck(uint32 ThreadId);

	/** Clears the stuck flag.
	 *  This is meant to be bound to the ThreadHeartBeat::OnThreadUnstuck delegate. Not all platforms register to save this flag. */
	CORE_API static void OnThreadUnstuck(uint32 ThreadId);

	/** Attempts to create the output report directory. */
	CORE_API static bool CreateCrashReportDirectory(const TCHAR* CrashGUIDRoot, int32 CrashIndex, FString& OutCrashDirectoryAbsolute);

	/** Notify the crash context exit has been requested. */
	CORE_API static void SetEngineExit(bool bIsRequestExit);

#if WITH_ADDITIONAL_CRASH_CONTEXTS
	/** Delegate for additional crash context. */
	static inline FAdditionalCrashContextDelegate& OnAdditionalCrashContextDelegate()
	{
		return AdditionalCrashContextDelegate;
	}

	/** Delegate for additional crash context. */
	static inline FAdditionalCrashContextDelegateEx& OnAdditionalCrashContextDelegateEx()
	{
		return AdditionalCrashContextDelegateEx;
	}
#endif //WITH_ADDITIONAL_CRASH_CONTEXTS

	/** Sets the process id to that has crashed. On supported platforms this will analyze the given process rather than current. Default is current process. */
	void SetCrashedProcess(const FProcHandle& Process)
	{
		ProcessHandle = Process;
	}

	/** Stores crashing thread id. */
	void SetCrashedThreadId(uint32 InId)
	{
		CrashedThreadId = InId;
	}

	/** Returns the crashing thread id. */
	uint32 GetCrashedThreadId() const
	{
		return CrashedThreadId;
	}

	/** Sets the number of stack frames to ignore when symbolicating from a minidump */
	CORE_API void SetNumMinidumpFramesToIgnore(int32 InNumMinidumpFramesToIgnore);

	/**
	 * Generate raw call stack for crash report (image base + offset) for the calling thread
	 * @param ErrorProgramCounter The program counter of where the occur occurred in the callstack being captured
	 * @param Context Optional thread context information
	 */
	CORE_API void CapturePortableCallStack(void* ErrorProgramCounter, void* Context);

	/**
	 * Generate raw call stack for crash report (image base + offset) for a different thread
	 * @param InThreadId The thread id of the thread to capture the callstack for
	 * @param Context Optional thread context information
	 */
	CORE_API void CaptureThreadPortableCallStack(const uint64 ThreadId, void* Context);

	/** Sets the portable callstack to a specified stack */
	CORE_API virtual void SetPortableCallStack(const uint64* StackFrames, int32 NumStackFrames);

	/** Gets the portable callstack to a specified stack and puts it into OutCallStack */
	CORE_API virtual void GetPortableCallStack(const uint64* StackFrames, int32 NumStackFrames, TArray<FCrashStackFrame>& OutCallStack) const;

	/** Store info about loaded modules */
	CORE_API virtual void CaptureModules();

	/** Gets info about loaded modules and stores it in the given array */
	CORE_API virtual void GetModules(TArray<FStackWalkModuleInfo>& OutModules) const;
	
	/** Adds a portable callstack for a thread */
	CORE_API virtual void AddPortableThreadCallStacks(TConstArrayView<FThreadCallStack> Threads);
	CORE_API virtual void AddPortableThreadCallStack(uint32 ThreadId, const TCHAR* ThreadName, const uint64* StackFrames, int32 NumStackFrames);

	/** Generates files required to be created in the crashing process */
	CORE_API static void GenerateFiles(const TCHAR* OutputDirectory);

	/** Allows platform implementations to copy files to report directory. */
	CORE_API virtual void CopyPlatformSpecificFiles(const TCHAR* OutputDirectory, void* Context);

	/** Cleanup platform specific files - called on startup, implemented per platform */
	CORE_API static void CleanupPlatformSpecificFiles();

	/**
	 * @return the type of this crash
	 */
	ECrashContextType GetType() const { return Type; }

	/**
	 * @return whether a crash context type is continable
	 */
	static bool IsTypeContinuable(ECrashContextType Type)
	{
		switch (Type)
		{
		// Verse runtime errors only halt the Verse runtime itself; they do not result in a crash.
		// Certain runtime errors may be recoverable from in the future.
		case ECrashContextType::VerseRuntimeError:
		case ECrashContextType::Ensure:
		case ECrashContextType::Stall:
			return true;

		case ECrashContextType::Crash:
		case ECrashContextType::Assert:
		case ECrashContextType::GPUCrash:
		case ECrashContextType::Hang:
		case ECrashContextType::OutOfMemory:
		case ECrashContextType::AbnormalShutdown:
		case ECrashContextType::Max:
		default:
			return false;
		}
	}

	/**
	 * Set the current deployment name (ie. EpicApp)
	 */
	CORE_API static void SetDeploymentName(const FString& EpicApp);

	/**
	 * Get the current deployment name (ie. EpicApp)
	 */
	CORE_API static const TCHAR* GetDeploymentName();

	/**
	 * Sets the type of crash triggered. Used to distinguish crashes caused for debugging purposes.
	 */
	CORE_API static void SetCrashTrigger(ECrashTrigger Type);

protected:
	/**
	 * @OutStr - a stream of Thread XML elements containing info (e.g. callstack) specific to an active thread
	 * @return - whether the operation was successful
	 */
	virtual bool GetPlatformAllThreadContextsString(FString& OutStr) const { return false; }

	FProcHandle ProcessHandle;
	ECrashContextType Type;
	uint32 CrashedThreadId;
	const TCHAR* ErrorMessage;
	int NumMinidumpFramesToIgnore;
	TArray<FCrashStackFrame> CallStack;
	TArray<FThreadStackFrames> ThreadCallStacks;
	TArray<FStackWalkModuleInfo> ModulesInfo;

	/** Allow platform implementations to provide a callstack property. Primarily used when non-native code triggers a crash. */
	CORE_API virtual const TCHAR* GetCallstackProperty() const;

	/** Get arbitrary engine data from the crash context */
	CORE_API static const FString* GetEngineData(const FString& Key);

	/** Get arbitrary game data from the crash context */
	CORE_API static const FString* GetGameData(const FString& Key);

	/** Generate a new crash guid base string. */
	CORE_API static FString GenerateNewCrashGuidBase();

private:

	/** Serializes the session context section of the crash context to a temporary file. */
	static void SerializeTempCrashContextToFile();

	/** Serializes the current user setting struct to a buffer. */
	static void SerializeUserSettings(FString& Buffer);

	/** Writes a common property to the buffer. */
	CORE_API static void AddCrashPropertyInternal(FString& Buffer, FStringView PropertyName, FStringView PropertyValue);

	/** Writes a common property to the buffer. */
	template <typename Type>
	static void AddCrashPropertyInternal(FString& Buffer, FStringView PropertyName, const Type& Value)
	{
		AddCrashPropertyInternal(Buffer, PropertyName, FStringView(TTypeToString<Type>::ToString(Value)));
	}

	/** Serializes platform specific properties to the buffer. */
	CORE_API virtual void AddPlatformSpecificProperties() const;

	/** Add callstack information to the crash report xml */
	void AddPortableCallStack() const;

	/** Produces a hash based on the offsets of the portable callstack and adds it to the xml */
	void AddPortableCallStackHash() const;

	/** Whether the given portable callstack frame should contribute to PCallStackHash. */
	CORE_API virtual bool ShouldIncludeInPortableCallStackHash(const FCrashStackFrame& Frame) const;

	/** Add GPU breadcrumbs information to the crash report xml */
	void AddGPUBreadcrumbs() const;

	/** Add module/pdb information to the crash report xml */
	void AddModules() const;

public:  // Allows this helper functionality to be present for clients to write their own types of crash reports.
	/** Writes header information to the buffer. */
	CORE_API static void AddHeader(FString& Buffer);

	/** Writes footer to the buffer. */
	CORE_API static void AddFooter(FString& Buffer);

	CORE_API static void BeginSection(FString& Buffer, const TCHAR* SectionName);
	CORE_API static void EndSection(FString& Buffer, const TCHAR* SectionName);
	CORE_API static void AddSection(FString& Buffer, const TCHAR* SectionName, const FString& SectionContent);

private:
	/** Called once when GConfig is initialized. Opportunity to cache values from config. */
	static void InitializeFromConfig();

	/** Called to update any localized strings in the crash context */
	static void UpdateLocalizedStrings();

	/**	Whether the Initialize() has been called */
	static bool bIsInitialized;

	/** The ID of the external process reporting crashes if the platform supports it and was configured to use it, zero otherwise (0 is a reserved system process ID, invalid for the out of process reporter). */
	CORE_API static uint32 OutOfProcessCrashReporterPid;

	/** The out of process crash reporter exit code, if available. The 32 MSB indicates if the exit code is set and the 32 LSB contains the exit code. The value can be read/write from different threads. */
	static volatile int64 OutOfProcessCrashReporterExitCode;

	/**	Static counter records how many crash contexts have been constructed */
	static int32 StaticCrashContextIndex;

#if WITH_ADDITIONAL_CRASH_CONTEXTS
	/** Delegate for additional crash context. */
	CORE_API static FAdditionalCrashContextDelegate AdditionalCrashContextDelegate;
	CORE_API static FAdditionalCrashContextDelegateEx AdditionalCrashContextDelegateEx;
#endif //WITH_ADDITIONAL_CRASH_CONTEXTS

	/** The buffer used to store the crash's properties. */
	mutable FString CommonBuffer;

	/**	Records which crash context we were using the StaticCrashContextIndex counter */
	int32 CrashContextIndex;

	/** Engine and game data set / reset delegates */
	CORE_API static FEngineDataResetDelegate OnEngineDataReset;
	CORE_API static FEngineDataSetDelegate OnEngineDataSet;

	CORE_API static FGameDataResetDelegate OnGameDataReset;
	CORE_API static FGameDataSetDelegate OnGameDataSet;

	// FNoncopyable
	FGenericCrashContext( const FGenericCrashContext& ) = delete;
	FGenericCrashContext& operator=(const FGenericCrashContext&) = delete;
};

struct FGenericMemoryWarningContext
{};

namespace RecoveryService
{
	/** Generates a name for the disaster recovery service embedded in the CrashReporterClientEditor. */
	CORE_API FString GetRecoveryServerName();

	/** Generates a name for the disaster recovery session. */
	CORE_API FString MakeSessionName();

	/** Tokenize the session name into its components. */
	CORE_API bool TokenizeSessionName(const FString& SessionName, FString* OutServerName, int32* SeqNum, FString* ProjName, FDateTime* DateTime);
}

#if WITH_ADDITIONAL_CRASH_CONTEXTS

/**
 * A thread local stack of callbacks that can be issued at time of the crash.
 */
struct FAdditionalCrashContextStack
{
	CORE_API static FAdditionalCrashContextStack& GetThreadContextProvider();
	CORE_API static void PushProvider(struct FScopedAdditionalCrashContextProvider* Provider);
	CORE_API static void PopProvider();

	CORE_API static void ExecuteProviders(FCrashContextExtendedWriter& Writer);

private:
	enum { MaxStackDepth = 16 };
	FAdditionalCrashContextStack* Next;
	const FScopedAdditionalCrashContextProvider* Stack[MaxStackDepth];
	uint32 StackIndex = 0;

	FAdditionalCrashContextStack();
	~FAdditionalCrashContextStack();
	
	inline void PushProviderInternal(const FScopedAdditionalCrashContextProvider* Provider) 
	{
		check(StackIndex < MaxStackDepth);
		Stack[StackIndex++] = Provider;
	}

	inline void PopProviderInternal()
	{
		check(StackIndex > 0);
		Stack[--StackIndex] = nullptr;
	}
};

struct FScopedAdditionalCrashContextProvider
{
public:
	FScopedAdditionalCrashContextProvider(TUniqueFunction<void(FCrashContextExtendedWriter&)> InFunc)
		: Func(MoveTemp(InFunc))
	{
		FAdditionalCrashContextStack::PushProvider(this);
	}

	~FScopedAdditionalCrashContextProvider()
	{
		FAdditionalCrashContextStack::PopProvider();
	}

	void Execute(FCrashContextExtendedWriter& Writer) const
	{
		Func(Writer);
	}

private:
	TUniqueFunction<void(FCrashContextExtendedWriter&)> Func;
};

#define UE_ADD_CRASH_CONTEXT_SCOPE(FuncExpr) FScopedAdditionalCrashContextProvider ANONYMOUS_VARIABLE(AddCrashCtx)(FuncExpr)

#else

#define UE_ADD_CRASH_CONTEXT_SCOPE(FuncExpr) 

#endif // WITH_ADDITIONAL_CRASH_CONTEXTS

