// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/Array.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Misc/DateTime.h"
#include "Misc/Optional.h"
#include "Misc/Parse.h"
#include "Misc/ConfigCacheIni.h"
#include "Internationalization/Regex.h"

// Represents a full driver version with multiple version numbers,
// e.g. 30.0.13023.4001.
class FDriverVersion
{
public:
	FDriverVersion() = default;

	FDriverVersion(const FString& DriverVersionString)
	{
		Parse(DriverVersionString);
	}

	void Parse(const FString& DriverVersionString)
	{
		Values.Empty();
		FString RemainingString = DriverVersionString;
		FString CurrentVersionNumber;

		uint32_t CurrentVersionNumberIdx = 0;
		while (RemainingString.Split(TEXT("."), &CurrentVersionNumber, &RemainingString))
		{
			Values.Add(FCString::Atoi(*CurrentVersionNumber));
		}
		Values.Add(FCString::Atoi(*RemainingString));
	}

	bool operator>(const FDriverVersion& Other) const
	{
		for (int32 Idx = 0; Idx < Values.Num(); ++Idx)
		{
			if (Values[Idx] > Other.Values[Idx])
			{
				return true;
			}
			if (Values[Idx] < Other.Values[Idx])
			{
				return false;
			}
		}

		return false;
	}

	bool operator==(const FDriverVersion& Other) const
	{
		return !(*this > Other) && !(Other > *this);
	}
	bool operator<(const FDriverVersion& Other) const
	{
		return Other > *this;
	}
	bool operator!=(const FDriverVersion& Other) const
	{
		return !(*this == Other);
	}
	bool operator>=(const FDriverVersion& Other) const
	{
		return (*this == Other) || (*this > Other);
	}
	bool operator<=(const FDriverVersion& Other) const
	{
		return (*this == Other) || (*this < Other);
	}

	// Index from the left (major version first).
	uint32 GetVersionValue(uint32 Index) const
	{
		return Values[Index];
	}

	const TArray<uint32>& GetVersionValues() const
	{
		return Values;
	}

private:
	TArray<uint32> Values;
};

enum EComparisonOp
{
	ECO_Unknown, ECO_Equal, ECO_NotEqual, ECO_Greater, ECO_GreaterOrEqual, ECO_Less, ECO_LessOrEqual
};

// Find the comparison operator in the input string, and advances the In pointer to point to the first character
// after the comparison operator.
// Defaults to Equal if no comparison operator is present.
inline EComparisonOp ParseComparisonOp(const TCHAR*& In)
{
	if (In[0] == '=' && In[1] == '=')
	{
		In += 2;
		return ECO_Equal;
	}
	if (In[0] == '!' && In[1] == '=')
	{
		In += 2;
		return ECO_NotEqual;
	}
	if (*In == '>')
	{
		++In;
		if (*In == '=')
		{
			++In;
			return ECO_GreaterOrEqual;
		}
		return ECO_Greater;
	}
	if (*In == '<')
	{
		++In;
		if (*In == '=')
		{
			++In;
			return ECO_LessOrEqual;
		}
		return ECO_Less;
	}
	return ECO_Equal;
}

// Compare two objects given the comparison operator.
template <class T>
bool CompareWithOp(const T& A, EComparisonOp Op, const T& B)
{
	switch (Op)
	{
		case ECO_Equal:			 return A == B;
		case ECO_NotEqual:		 return A != B;
		case ECO_Greater:		 return A > B;
		case ECO_GreaterOrEqual: return A >= B;
		case ECO_Less:		     return A < B;
		case ECO_LessOrEqual:	 return A <= B;
	}
	check(0);
	return false;
}

struct FGPUDriverInfo
{
	// Hardware vendor ID, e.g. 0x10DE for NVIDIA.
	uint32 VendorId = 0;

	// Hardware adapter generation (vendor-specific).
	uint32 AdapterGeneration = 0;

	// Name of the graphics device, e.g. "NVIDIA GeForce GTX 680" or "AMD Radeon R9 200 / HD 7900 Series".
	FString DeviceDescription;

	// Name of the hardware vendor, e.g. "NVIDIA" or "Advanced Micro Devices, Inc."
	FString ProviderName;

	// Internal driver version, which may differ from the version shown to the user. Set to "Unknown" if the driver detection failed.
	FString InternalDriverVersion;

	// User-facing driver version.
	FString UserDriverVersion;

	// Driver date as reported by the driver. Format is MM-DD-YYYY.
	FString DriverDate;

	// Current RHI.
	FString RHIName;

	bool IsValid() const
	{
		return !DeviceDescription.IsEmpty()
			&& VendorId
			&& (InternalDriverVersion != TEXT("Unknown")) // If driver detection code fails.
			&& (InternalDriverVersion != TEXT(""));		  // If running on a non-Windows platform we don't fill in the driver version.
	}

	void SetAMD() { VendorId = 0x1002; }
	void SetIntel() { VendorId = 0x8086; }
	void SetNVIDIA() { VendorId = 0x10DE; }
	void SetQualcomm() { VendorId = 0x4D4F4351; }

	bool IsAMD() const { return VendorId == 0x1002; }
	bool IsIntel() const { return VendorId == 0x8086; }
	bool IsNVIDIA() const { return VendorId == 0x10DE; }
	bool IsQualcomm() const
	{
		// On Windows, the DXGI vendor ID is reported as 0x4D4F4351.
		// On other platforms, it's 0x5143.
		return VendorId == 0x5143 || VendorId == 0x4D4F4351;
	}

	bool IsAdapterGenerationKnown() const
	{
		return AdapterGeneration != 0;
	}

	bool IsSameDriverVersionGeneration(const FDriverVersion& UnifiedDriverVersion) const
	{
		if (IsIntel())
		{
			FDriverVersion ThisVersion(GetUnifiedDriverVersion());

			// https://www.intel.com/content/www/us/en/support/articles/000005654/graphics.html
			// Version format changed in April 2018 starting with xx.xx.100.xxxx.
			// In the unified driver version, that's the first value.
			return (ThisVersion.GetVersionValue(0) >= 100) == (UnifiedDriverVersion.GetVersionValue(0) >= 100);
		}

		return true;
	}

	static FString GetNVIDIAUnifiedVersion(const FString& InternalVersion)
	{
		// Ignore the Windows/DirectX version by taking the last digits of the internal version
		// and moving the version dot. Coincidentally, that's the user-facing string. For example:
		// 9.18.13.4788 -> 3.4788 -> 347.88
		if (InternalVersion.Len() < 6)
		{
			return InternalVersion;
		}

		FString RightPart = InternalVersion.Right(6);
		RightPart = RightPart.Replace(TEXT("."), TEXT(""));
		RightPart.InsertAt(3, TEXT("."));
		return RightPart;
	}

	static FString GetIntelUnifiedVersion(const FString& InternalVersion)
	{
		// https://www.intel.com/content/www/us/en/support/articles/000005654/graphics.html
		// Drop off the OS and DirectX version. For example:
		// 27.20.100.8935 -> 100.8935
		int32 DotIndex = InternalVersion.Find(TEXT("."), ESearchCase::CaseSensitive, ESearchDir::FromStart);
		if (DotIndex != INDEX_NONE)
		{
			DotIndex = InternalVersion.Find(TEXT("."), ESearchCase::CaseSensitive, ESearchDir::FromStart, DotIndex + 1);
			if (DotIndex != INDEX_NONE)
			{
				return InternalVersion.RightChop(DotIndex + 1);
			}
		}
		return InternalVersion;
	}

	FString GetUnifiedDriverVersion() const
	{
		// We use the internal version as the base to avoid instability or changes to the user-facing driver versioning scheme,
		// and we remove the parts of the version that identify unimportant factors like Windows or DirectX version.
		const FString& FullVersion = InternalDriverVersion;

		if (IsNVIDIA() && (InternalDriverVersion != UserDriverVersion))
		{
			return GetNVIDIAUnifiedVersion(FullVersion);
		}
		else if (IsIntel())
		{
			return GetIntelUnifiedVersion(FullVersion);
		}
		return FullVersion;
	}
};

static FDateTime ParseMonthDayYearDate(const FString& DateString)
{
	FString Month, DayYear, Day, Year;
	DateString.Split(TEXT("-"), &Month, &DayYear);
	DayYear.Split(TEXT("-"), &Day, &Year);
	return FDateTime(FCString::Atoi(*Year), FCString::Atoi(*Month), FCString::Atoi(*Day));
}

// Represents an optional set of constraints that a driver configuration entry can include:
//   * The RHI being used (e.g. "D3D12" will result in the entry to be considered only when running in D3D12).
//   * The adapter name, using a regular expression (e.g. ".*RTX.*" will result in the entry to be considered only for cards that contain the string "RTX").
//   * The adapter generation. This is a vendor-specific integer and may not be implemented for all vendors.
struct FDriverConfigEntryConstraints {
	struct FIntegerConstraint {
		EComparisonOp ComparisonOp;
		uint32 Value;

		static FIntegerConstraint FromString(const FString& IntegerConstraintString)
		{
			FIntegerConstraint Constraint;
			const TCHAR* Chars = *IntegerConstraintString;
			Constraint.ComparisonOp = ParseComparisonOp(Chars);
			Constraint.Value = FCString::Atoi(Chars);
			return Constraint;
		}
	};

	TOptional<FString> RHINameConstraint;
	TOptional<FRegexPattern> AdapterNameRegexConstraint;
	TOptional<FIntegerConstraint> AdapterGenerationConstraint;

	FDriverConfigEntryConstraints() = default;

	FDriverConfigEntryConstraints(const TCHAR* Entry)
	{
		FString RHINameString;
		FParse::Value(Entry, TEXT("RHI="), RHINameString);
		if (!RHINameString.IsEmpty())
		{
			RHINameConstraint = RHINameString;
		}

		FString AdapterNameRegexString;
		FParse::Value(Entry, TEXT("AdapterNameRegex="), AdapterNameRegexString);
		if (!AdapterNameRegexString.IsEmpty())
		{
			AdapterNameRegexConstraint.Emplace(AdapterNameRegexString);
		}

		FString AdapterGenerationString;
		FParse::Value(Entry, TEXT("AdapterGeneration="), AdapterGenerationString);
		if (!AdapterGenerationString.IsEmpty())
		{
			AdapterGenerationConstraint = FIntegerConstraint::FromString(AdapterGenerationString);
		}
	}

	bool AreConstraintsSatisfied(const FGPUDriverInfo& Info, uint32& OutNumSatisfiedConstraints) const
	{
		OutNumSatisfiedConstraints = 0;

		if (RHINameConstraint)
		{
			if (*RHINameConstraint != Info.RHIName)
			{
				return false;
			}
			OutNumSatisfiedConstraints++;
		}

		if (AdapterNameRegexConstraint)
		{
			FRegexMatcher Matcher(*AdapterNameRegexConstraint, Info.DeviceDescription);
			if (!Matcher.FindNext())
			{
				return false;
			}
			OutNumSatisfiedConstraints++;
		}

		if (AdapterGenerationConstraint)
		{
			if (!Info.IsAdapterGenerationKnown() || !CompareWithOp(Info.AdapterGeneration, AdapterGenerationConstraint->ComparisonOp, AdapterGenerationConstraint->Value))
			{
				return false;
			}
			OutNumSatisfiedConstraints++;
		}

		return true;
	}

	bool HasConstraints()
	{
		return RHINameConstraint || AdapterNameRegexConstraint || AdapterGenerationConstraint;
	}
};

// This corresponds to one DriverDenyList entry in the Hardware.ini configuration file. One entry includes:
//   * The drivers being denylisted, specified through one of the following:
//      * A driver version, along with any comparison operator (e.g. "<473.47" would denylist any driver which version is less than 473.47). This uses the "unified" driver version (see FGPUDriverInfo::GetUnifiedDriverVersion).
//      * A driver date, along with any comparison operator (e.g. "<=10-31-2023" would denylist any driver which release date is on or before Oct. 31, 2023).
//   * A set of optional constraints that the denylist entry will consider when the driver is being tested (see FDriverConfigEntryConstraints).
//   * The reason for the driver(s) being denylisted, e.g.visual glitches, stability, or performance issues.
struct FDriverDenyListEntry : public FDriverConfigEntryConstraints {
	struct FDriverDateDenyListEntry {
		EComparisonOp ComparisonOp;
		FDateTime Date;

		// Expected format is MM-DD-YYYY.
		static FDriverDateDenyListEntry FromString(const FString& DriverDateStringWithComparisonOp)
		{
			FDriverDateDenyListEntry Entry;
			const TCHAR* Chars = *DriverDateStringWithComparisonOp;
			Entry.ComparisonOp = ParseComparisonOp(Chars);
			Entry.Date = ParseMonthDayYearDate(FString(Chars));
			return Entry;
		}
	};

	struct FDriverVersionDenyListEntry {
		EComparisonOp ComparisonOp;
		FDriverVersion Version;

		static FDriverVersionDenyListEntry FromString(const FString& DriverVersionStringWithComparisonOp)
		{
			FDriverVersionDenyListEntry Entry;
			const TCHAR* Chars = *DriverVersionStringWithComparisonOp;
			Entry.ComparisonOp = ParseComparisonOp(Chars);
			Entry.Version.Parse(Chars);
			return Entry;
		}
	};

	FDriverDenyListEntry() = default;

	FDriverDenyListEntry(const TCHAR* Entry)
		: FDriverConfigEntryConstraints(Entry)
	{
		FString DriverVersionString;
		FParse::Value(Entry, TEXT("DriverVersion="), DriverVersionString);
		if (!DriverVersionString.IsEmpty())
		{
			DriverVersion = FDriverVersionDenyListEntry::FromString(DriverVersionString);
		}

		FString DriverDateString;
		FParse::Value(Entry, TEXT("DriverDate="), DriverDateString);
		if (!DriverDateString.IsEmpty())
		{
			DriverDate = FDriverDateDenyListEntry::FromString(DriverDateString);
		}

		ensureMsgf(IsValid(), TEXT("Exactly one of driver date or driver version must be specified in a driver denylist entry"));

		FParse::Value(Entry, TEXT("Reason="), DenylistReason);
		ensure(!DenylistReason.IsEmpty());
	}

	// Checks whether the given driver is denylisted by this entry, and also returns the number
	// of constraints that have been satisfied in OutNumSatisfiedConstraints.
	bool AppliesToDriver(const FGPUDriverInfo& Info, uint32& OutNumSatisfiedConstraints) const
	{
		OutNumSatisfiedConstraints = 0;

		if (!IsValid())
		{
			return false;
		}

		// Check constraints first. Only apply the driver version/date check
		// if all constraints are satisfied.
		if (!AreConstraintsSatisfied(Info, OutNumSatisfiedConstraints))
		{
			return false;
		}

		if (DriverVersion && Info.IsSameDriverVersionGeneration(DriverVersion->Version))
		{
			FDriverVersion CurrentDriverVersion(Info.GetUnifiedDriverVersion());
			return CompareWithOp(CurrentDriverVersion, DriverVersion->ComparisonOp, DriverVersion->Version);
		}

		if (DriverDate)
		{
			FDateTime CurrentDriverDate = ParseMonthDayYearDate(Info.DriverDate);
			return CompareWithOp(CurrentDriverDate, DriverDate->ComparisonOp, DriverDate->Date);
		}

		return false;
	}

	// Returns true whether this denylist entry will always apply to the latest drivers available,
	// i.e. the comparison function is > or >=.
	bool AppliesToLatestDrivers()
	{
		if (DriverVersion)
		{
			return DriverVersion->ComparisonOp == ECO_Greater || DriverVersion->ComparisonOp == ECO_GreaterOrEqual;
		}
		if (DriverDate)
		{
			return DriverDate->ComparisonOp == ECO_Greater || DriverDate->ComparisonOp == ECO_GreaterOrEqual;
		}
		return false;
	}

	bool IsValid() const
	{
		return DriverVersion.IsSet() != DriverDate.IsSet();
	}

	TOptional<FDriverVersionDenyListEntry> DriverVersion;
	TOptional<FDriverDateDenyListEntry> DriverDate;

	FString	DenylistReason;
};

// This corresponds to one SuggestedDriverVersion entry in the Hardware.ini configuration file. One entry includes:
//   * The suggested driver version.
//   * A set of optional constraints (see FDriverConfigEntryConstraints).
struct FSuggestedDriverEntry : public FDriverConfigEntryConstraints {

	FSuggestedDriverEntry() = default;
	FSuggestedDriverEntry(const TCHAR* Entry)
		: FDriverConfigEntryConstraints(Entry)
	{
		// For backwards compatibility, accept either "DriverVersion=..." or
		// just the version string without the structured format.
		FParse::Value(Entry, TEXT("DriverVersion="), SuggestedDriverVersion);
		if (SuggestedDriverVersion.IsEmpty() && !HasConstraints() && !FString(Entry).Contains(TEXT("=")))
		{
			SuggestedDriverVersion = Entry;
		}
	}

	// Checks whether this entry applies to the given driver info, and also returns the number
	// of constraints that have been satisfied in OutNumSatisfiedConstraints.
	bool AppliesToDriver(const FGPUDriverInfo& Info, uint32& OutNumSatisfiedConstraints)
	{
		return IsValid() && AreConstraintsSatisfied(Info, OutNumSatisfiedConstraints);
	}

	bool IsValid()
	{
		return !SuggestedDriverVersion.IsEmpty();
	}

	FString SuggestedDriverVersion;
};

// This corresponds to one SuggestedDriverList entry in the Hardware.ini configuration file. One entry includes:
//   * A driver version
//   * A set of optional constraints that the entry will consider when the driver is being tested (see FDriverConfigEntryConstraints).
struct FSuggestedDriverListEntry : public FDriverConfigEntryConstraints {
	FSuggestedDriverListEntry() = default;

	FSuggestedDriverListEntry(const TCHAR* Entry)
		: FDriverConfigEntryConstraints(Entry)
	{
		FParse::Value(Entry, TEXT("DriverVersion="), DriverVersion);

		ensureMsgf(IsValid(), TEXT("DriverVersion must be specified in a suggested driver list entry"));
	}

	// Checks whether this entry applies to the given driver info, and also returns the number
	// of constraints that have been satisfied in OutNumSatisfiedConstraints.
	bool AppliesToDriver(const FGPUDriverInfo& Info, uint32& OutNumSatisfiedConstraints) const
	{
		OutNumSatisfiedConstraints = 0;

		return IsValid() && AreConstraintsSatisfied(Info, OutNumSatisfiedConstraints);
	}

	bool IsValid() const
	{
		return !DriverVersion.IsEmpty();
	}

	FString	DriverVersion;
};

class FGPUDriverHelper
{
public:
	FGPUDriverHelper(const FGPUDriverInfo& InDriverInfo)
		: DriverInfo(InDriverInfo)
	{
	}

	// Returns the best suggested driver version that matches the current configuration, if it exists. 
	FString FindSuggestedDriverVersion() const
	{
		TOptional<FSuggestedDriverListEntry> ListEntry = GetBestConfigEntry<FSuggestedDriverListEntry>(TEXT("SuggestedDriverList"));
		if (ListEntry.IsSet())
		{
			return ListEntry->DriverVersion;
		}

		TOptional<FSuggestedDriverEntry> Entry = GetBestConfigEntry<FSuggestedDriverEntry>(TEXT("SuggestedDriverVersion"));
		if (Entry.IsSet())
		{
			return Entry->SuggestedDriverVersion;
		}

		return FString();
	}

	// Finds the best driver denylist entry that matches the current driver and configuration, if it exists. 
	TOptional<FDriverDenyListEntry> FindDriverDenyListEntry() const
	{
		return GetBestConfigEntry<FDriverDenyListEntry>(TEXT("DriverDenyList"));
	}

private:
	template <typename TEntryType>
	TOptional<TEntryType> GetBestConfigEntry(const TCHAR* ConfigEntryName) const
	{
		const FString Section = GetVendorSectionName();

		if (Section.IsEmpty())
		{
			return {};
		}

		// Multiple entries can match, but some entries may match on more
		// constraints (e.g. both RHI and adapter name). Choose the entry
		// that satisfies the most constraints.
		TPair<TEntryType, uint32> MostRelevantEntry;

		TArray<FString> EntryStrings;
		GConfig->GetArray(*Section, ConfigEntryName, EntryStrings, GHardwareIni);
		for (const FString& EntryString : EntryStrings)
		{
			ensure(!EntryString.IsEmpty());
			TEntryType Entry(*EntryString);

			uint32 NumSatisfiedConstraints = 0;
			if (Entry.AppliesToDriver(DriverInfo, NumSatisfiedConstraints))
			{
				if (NumSatisfiedConstraints >= MostRelevantEntry.Value)
				{
					MostRelevantEntry = { Entry, NumSatisfiedConstraints };
				}
			}
		}

		if (MostRelevantEntry.Key.IsValid())
		{
			return MostRelevantEntry.Key;
		}
		return {};
	}

	// Get the corresponding vendor's section name in the Hardware.ini file.
	// Returns an empty string if not found.
	FString GetVendorSectionName() const
	{
		const TCHAR* Section = nullptr;

		if (DriverInfo.IsNVIDIA())
		{
			Section = TEXT("GPU_NVIDIA");
		}
		if (DriverInfo.IsAMD())
		{
			Section = TEXT("GPU_AMD");
		}
		else if (DriverInfo.IsIntel())
		{
			Section = TEXT("GPU_Intel");
		}
		else if (DriverInfo.IsQualcomm())
		{
			Section = TEXT("GPU_Qualcomm");
		}
		if (!Section)
		{
			return TEXT("");
		}

		return FString::Printf(TEXT("%s %s"), Section, ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName()));
	}

	FGPUDriverInfo DriverInfo;
};
