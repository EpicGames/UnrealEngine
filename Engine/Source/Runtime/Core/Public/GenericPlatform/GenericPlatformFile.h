// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	GenericPlatformFile.h: Generic platform file interfaces
==============================================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PlatformMemory.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/DateTime.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"
#include "Templates/ValueOrError.h"

class FArchive;
class IMappedFileHandle;
class IAsyncReadFileHandle;
namespace ELogVerbosity { enum Type : uint8; }

/**
* Enum for async IO priorities. 
*/
enum EAsyncIOPriorityAndFlags : int32
{
	AIOP_PRIORITY_MASK = 0x000000ff,

	// Flags - combine with priorities if needed
	AIOP_FLAG_PRECACHE			= 0x00000100,
	AIOP_FLAG_DONTCACHE			= 0x00000200,
	// Use it to specify memory where read from cpu can be slow
	AIOP_FLAG_HW_TARGET_MEMORY	= 0x00000400,	

	// Priorities
	AIOP_MIN = 0,
	AIOP_Low,
	AIOP_BelowNormal,
	AIOP_Normal,
	AIOP_High,
	AIOP_CriticalPath,
	AIOP_MAX = AIOP_CriticalPath,
	AIOP_NUM,

	// Legacy (for back-compat). Better to specify priority and AIOP_FLAG_PRECACHE separately
	AIOP_Precache = AIOP_MIN | AIOP_FLAG_PRECACHE,
};
ENUM_CLASS_FLAGS(EAsyncIOPriorityAndFlags);

/**
 * Enum for platform file read flags
 */
enum class EPlatformFileRead : uint8
{
	None = 0x0,
	AllowWrite = 0x01	// attempts to open for read while allowing others to write
};

ENUM_CLASS_FLAGS(EPlatformFileRead);

/**
 * Enum for platform file write flags
 */
enum class EPlatformFileWrite : uint8
{
	None = 0x0,
	AllowRead = 0x01,	// attempts to open for write while allowing others to read
	AttemptDeleteAndCreate = 0x02 // if the target file cannot be open for write, attempt to delete it and create a new one in it splace
};

ENUM_CLASS_FLAGS(EPlatformFileWrite);

/**
 * Enum for the DirectoryVisitor flags
 */
enum class EDirectoryVisitorFlags : uint8
{
	None = 0x0,
	ThreadSafe = 0x01	// should be set when the Visit function can be called from multiple threads at once.
};

/**
 * Enum for IsSymlink() results (added so this func can return ESymlinkResult::Unimplemented)
 */
enum class ESymlinkResult : int8
{
	Unimplemented = -1,
	NonSymlink = 0,
	Symlink = 1,
};

ENUM_CLASS_FLAGS(EDirectoryVisitorFlags);

/** Results that can be returned from IPlatformFile FileJournal API. */
enum class EFileJournalResult
{
	Success,
	InvalidPlatform,
	InvalidVolumeName,
	JournalNotActive,
	JournalWrapped,
	FailedOpenJournal,
	FailedDescribeJournal,
	FailedReadJournal,
	JournalInternalError,
	UnhandledJournalVersion,
};

/** 
 * File handle interface. 
**/
class IFileHandle
{
public:
	/** Destructor, also the only way to close the file handle **/
	virtual ~IFileHandle() = default;

	/** Return the current write or read position. **/
	virtual int64		Tell() = 0;
	/** 
	 * Change the current write or read position. 
	 * @param NewPosition	new write or read position
	 * @return				true if the operation completed successfully.
	**/
	virtual bool		Seek(int64 NewPosition) = 0;

	/** 
	 * Change the current write or read position, relative to the end of the file.
	 * @param NewPositionRelativeToEnd	new write or read position, relative to the end of the file should be <=0!
	 * @return							true if the operation completed successfully.
	**/
	virtual bool		SeekFromEnd(int64 NewPositionRelativeToEnd = 0) = 0;

	/** 
	 * Read bytes from the file.
	 * @param Destination	Buffer to holds the results, should be at least BytesToRead in size.
	 * @param BytesToRead	Number of bytes to read into the destination.
	 * @return				true if the operation completed successfully. Returns false if a read is attempted
	 *						while the read position is already at the end of the file.
	 * @remark				If BytesToRead is larger than the bytes remaining in the file from the current read position
	 *						the Read() will return true if the underlying file read could read all remaining bytes.
	**/
	virtual bool		Read(uint8* Destination, int64 BytesToRead) = 0;

	virtual bool		ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset) = 0;

	/** 
	 * Write bytes to the file.
	 * @param Source		Buffer to write, should be at least BytesToWrite in size.
	 * @param BytesToWrite	Number of bytes to write.
	 * @return				true if the operation completed successfully.
	**/
	virtual bool		Write(const uint8* Source, int64 BytesToWrite) = 0;

	/**
	 * Flushes file handle to disk.
	 * @param bFullFlush	true to flush everything about the file (including its meta-data) with a strong guarantee that it will be on disk by the time this function returns, 
	 *						or false to let the operating/file system have more leeway about when the data actually gets written to disk
	 * @return				true if operation completed successfully.
	**/
	virtual bool		Flush(const bool bFullFlush = false) = 0;

	/** 
	 * Truncate the file to the given size (in bytes).
	 * @param NewSize		Truncated file size (in bytes).
	 * @return				true if the operation completed successfully.
	**/
	virtual bool		Truncate(int64 NewSize) = 0;

	/**
	 * Minimizes optional system or process cache kept for the file.
	**/
	virtual void		ShrinkBuffers()
	{
	}

public:
	/////////// Utility Functions. These have a default implementation that uses the pure virtual operations.

	/** Return the total size of the file **/
	CORE_API virtual int64		Size();
};


/**
 * Contains the information that's returned from stat'ing a file or directory 
 */
struct FFileStatData
{
	FFileStatData()
		: CreationTime(FDateTime::MinValue())
		, AccessTime(FDateTime::MinValue())
		, ModificationTime(FDateTime::MinValue())
		, FileSize(-1)
		, bIsDirectory(false)
		, bIsReadOnly(false)
		, bIsValid(false)
	{
	}

	FFileStatData(FDateTime InCreationTime, FDateTime InAccessTime,	FDateTime InModificationTime, const int64 InFileSize, const bool InIsDirectory, const bool InIsReadOnly)
		: CreationTime(InCreationTime)
		, AccessTime(InAccessTime)
		, ModificationTime(InModificationTime)
		, FileSize(InFileSize)
		, bIsDirectory(InIsDirectory)
		, bIsReadOnly(InIsReadOnly)
		, bIsValid(true)
	{
	}

	/** The time that the file or directory was originally created, or FDateTime::MinValue if the creation time is unknown */
	FDateTime CreationTime;

	/** The time that the file or directory was last accessed, or FDateTime::MinValue if the access time is unknown */
	FDateTime AccessTime;

	/** The time the the file or directory was last modified, or FDateTime::MinValue if the modification time is unknown */
	FDateTime ModificationTime;

	/** Size of the file (in bytes), or -1 if the file size is unknown */
	int64 FileSize;

	/** True if this data is for a directory, false if it's for a file */
	bool bIsDirectory : 1;

	/** True if this file is read-only */
	bool bIsReadOnly : 1;

	/** True if file or directory was found, false otherwise. Note that this value being true does not ensure that the other members are filled in with meaningful data, as not all file systems have access to all of this data */
	bool bIsValid : 1;
};


/**
 * A handle used by the FileJournal API. Platform-specific identifier for which disk journal is being read.
 */
typedef uint64 FFileJournalId;
inline constexpr FFileJournalId FileJournalIdInvalid = static_cast<FFileJournalId>(MAX_uint64);

/**
 * A handle used by the FileJournal API. Represents an entry for an action on a file in the FileJournal.
 */
typedef uint64 FFileJournalEntryHandle;
inline constexpr FFileJournalEntryHandle FileJournalEntryHandleInvalid = static_cast<FFileJournalEntryHandle>(MAX_uint64);

/**
 * A handle used by the FileJournal API. Uniquely represents a file on disk without needing to use the filename.
 */
struct FFileJournalFileHandle
{
	bool operator==(const FFileJournalFileHandle& Other) const;
	bool operator!=(const FFileJournalFileHandle& Other) const;
	CORE_API FString ToString();

	uint8 Bytes[20];
};
uint32 GetTypeHash(const FFileJournalFileHandle&);
CORE_API extern const FFileJournalFileHandle FileJournalFileHandleInvalid;

/**
 * Contains the information that's returned from FileJournalGetFileData for a file.
 */
struct FFileJournalData
{
	FFileJournalData();

	FDateTime ModificationTime;
	FFileJournalFileHandle JournalHandle = FileJournalFileHandleInvalid;
	bool bIsValid : 1;
	bool bIsDirectory : 1;
	bool bIsReparsePoint : 1;
	bool bIsReadOnly : 1;
};

/** Stores custom error messages from the engine along with an optional system error code that can provider more detailed information */
class FFileSystemError
{
public:
	FFileSystemError() = delete;

	FFileSystemError(const FFileSystemError&) = delete;
	FFileSystemError(FFileSystemError&&) = default;	
	FFileSystemError& operator=(const FFileSystemError&) = delete;
	FFileSystemError& operator=(FFileSystemError&&) = default;

	explicit FFileSystemError(FString&& InErrorMessage, int32 InSystemErrorCode = 0)
		: ErrorMessage(MoveTemp(InErrorMessage))
		, SystemErrorCode(InSystemErrorCode)
	{
	}

	/** Return the error message, if a valid system error code was provided then this will be appended to the end of the message */
	FString GetMessage()
	{
		if (SystemErrorCode != 0)
		{
			TCHAR FormattedErrorMsg[MAX_SPRINTF] = { 0 };
			FPlatformMisc::GetSystemErrorMessage(FormattedErrorMsg, UE_ARRAY_COUNT(FormattedErrorMsg), SystemErrorCode);

			return FString::Printf(TEXT("%s [%s (%d)]"), *ErrorMessage, FormattedErrorMsg, SystemErrorCode);
		}
		else
		{
			return ErrorMessage;
		}
	}

	/** Return the error message, if a valid system error code was provided then this will be appended to the end of the message */
	template <typename CharType>
	friend inline TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const FFileSystemError& Error)
	{
		TCHAR FormattedErrorMsg[MAX_SPRINTF] = { 0 };
		FPlatformMisc::GetSystemErrorMessage(FormattedErrorMsg, UE_ARRAY_COUNT(FormattedErrorMsg), Error.SystemErrorCode);

		Builder << Error.ErrorMessage << TEXT(" [") << FormattedErrorMsg << TEXT(" (") << Error.SystemErrorCode << TEXT(")]");
		return Builder;
	}

private:
	FString ErrorMessage;
	int32 SystemErrorCode;
};

/** Data structure returned by calls to IPlatformFile::OpenRead */
using FFileOpenResult = TValueOrError<TUniquePtr<IFileHandle>, FFileSystemError>;
using FFileOpenAsyncResult = TValueOrError<TUniquePtr<IAsyncReadFileHandle>, FFileSystemError>;

/** Data structure returned by calls to IPlatformFile::OpenMapped */
using FOpenMappedResult = TValueOrError<TUniquePtr<IMappedFileHandle>, FFileSystemError>;

/**
* File I/O Interface
**/
class IPlatformFile
{
public:
	/** Physical file system of the _platform_, never wrapped. **/
	static CORE_API IPlatformFile& GetPlatformPhysical();
	/** Returns the name of the physical platform file type. */
	static CORE_API const TCHAR* GetPhysicalTypeName();
	/** Constructor/Destructor. */
	CORE_API IPlatformFile();
	CORE_API virtual ~IPlatformFile();

	/**
	 *	Set whether the sandbox is enabled or not
	 *
	 *	@param	bInEnabled		true to enable the sandbox, false to disable it
	 */
	virtual void SetSandboxEnabled(bool bInEnabled)
	{
	}

	/**
	 *	Returns whether the sandbox is enabled or not
	 *
	 *	@return	bool			true if enabled, false if not
	 */
	virtual bool IsSandboxEnabled() const
	{
		return false;
	}

	/**
	 * Checks if this platform file should be used even though it was not asked to be.
	 * i.e. pak files exist on disk so we should use a pak file
	 */
	virtual bool ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const
	{
		return false;
	}

	/**
	 * Initializes platform file.
	 *
	 * @param Inner Platform file to wrap by this file.
	 * @param CmdLine Command line to parse.
	 * @return true if the initialization was successful, false otherise. */
	virtual bool		Initialize(IPlatformFile* Inner, const TCHAR* CmdLine) = 0;

	/**
	 * Performs initialization of the platform file after it has become the active (FPlatformFileManager.GetPlatformFile() will return this
	 */
	virtual void		InitializeAfterSetActive() { }

	/**
	 * Performs initialization of the platform file after the project path has been set.
	 */
	virtual void		InitializeAfterProjectFilePath() { }

	/**
	 * Build an in memory unique pak file from a subset of files in this pak file
	 */
	virtual void		MakeUniquePakFilesForTheseFiles(const TArray<TArray<FString>>& InFiles) { }

	/**
	* Performs initialization of the platform file after the new async IO has been enabled
	*/
	virtual void		InitializeNewAsyncIO() { }

	/**
	 * Identifies any platform specific paths that are guaranteed to be local (i.e. cache, scratch space)
	 */
	virtual void		AddLocalDirectories(TArray<FString> &LocalDirectories)
	{
		if (GetLowerLevel())
		{
			GetLowerLevel()->AddLocalDirectories(LocalDirectories);
		}
	}

	virtual void		BypassSecurity(bool bInBypass)
	{
		if (GetLowerLevel() != nullptr)
		{
			GetLowerLevel()->BypassSecurity(bInBypass);
		}
	}

	/** Platform file can override this to get a regular tick from the engine */
	virtual void Tick() { }
	/** Gets the platform file wrapped by this file. */
	virtual IPlatformFile* GetLowerLevel() = 0;
	/** Sets the platform file wrapped by this file. */
	virtual void SetLowerLevel(IPlatformFile* NewLowerLevel) = 0;
	/** Gets this platform file type name. */
	virtual const TCHAR* GetName() const = 0;
	/** Return true if the file exists. **/
	virtual bool		FileExists(const TCHAR* Filename) = 0;
	/** Return the size of the file, or -1 if it doesn't exist. **/
	virtual int64		FileSize(const TCHAR* Filename) = 0;
	/** Delete a file and return true if the file exists. Will not delete read only files. **/
	virtual bool		DeleteFile(const TCHAR* Filename) = 0;
	/** Delete an array of files and return true if ALL deletes are succeeded. **/
	virtual bool		DeleteFiles(const TArrayView<const TCHAR*>& Filenames)
	{
		bool bOneFailed = false;

		for (const TCHAR* File : Filenames)
		{
			bOneFailed |= !DeleteFile(File);
		}

		return !bOneFailed;
	}
	/** Return true if the file is read only. **/
	virtual bool		IsReadOnly(const TCHAR* Filename) = 0;
	/** Attempt to move a file. Return true if successful. Will not overwrite existing files. **/
	virtual bool		MoveFile(const TCHAR* To, const TCHAR* From) = 0;
	/** Attempt to change the read only status of a file. Return true if successful. **/
	virtual bool		SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) = 0;
	/** Return the modification time of a file. Returns FDateTime::MinValue() on failure **/
	virtual FDateTime	GetTimeStamp(const TCHAR* Filename) = 0;
	/** Sets the modification time of a file **/
	virtual void		SetTimeStamp(const TCHAR* Filename, FDateTime DateTime) = 0;
	/** Return the last access time of a file. Returns FDateTime::MinValue() on failure **/
	virtual FDateTime	GetAccessTimeStamp(const TCHAR* Filename) = 0;
	/** For case insensitive filesystems, returns the full path of the file with the same case as in the filesystem */
	virtual FString GetFilenameOnDisk(const TCHAR* Filename) = 0;

	/** Return ESymlinkResult::Symlink if the file is a symbolic link, if unimplemented, try to go down a level */
	virtual ESymlinkResult IsSymlink(const TCHAR* Filename) 
	{ 
		if (GetLowerLevel())
		{
			return GetLowerLevel()->IsSymlink(Filename);
		}
		return ESymlinkResult::Unimplemented; 
	}

	/**
	 * Determine if the file has been downloaded from a web browser, based on platform-specific metadata.
	 *
	 * @param Filename The file to check if it has a mark of the web.
	 * @param OutSourceURL An optional pointer to return a source url if available.
	 *
	 * @return true if the file has a mark of the web, false otherwise.
	 * @remark Only works for files on disk and currently only implemented for Windows.
	 */
	virtual bool HasMarkOfTheWeb(FStringView Filename, FString* OutSourceURL = nullptr) { return false; }

	/**
	 * Attempt to change the platform-specific metadata that indicates if the file has been downloaded from a web browser.
	 *
	 * @param Filename The file to change it's mark of the web status.
	 * @param bNewStatus New mark of the web status for the file.
	 * @param InSourceURL An optional pointer to a source url that will be applied if new status true.
	 *
	 * @return true if the file's mark of the web status was successful changed, false otherwise.
	 * @remark Only works for files on disk and currently only implemented for Windows.
	 */
	virtual bool SetMarkOfTheWeb(FStringView Filename, bool bNewStatus, const FString* InSourceURL = nullptr) { return false; }

	/** Flags to be used when opening a file for reading via IPlatformFile::OpenRead */
	enum class EOpenReadFlags : uint8
	{
		None = 0,
		/** Allow other handles/processes to write to this file. This flag is needed to open files that are currently being written to as well */
		AllowWrite	= 1 << 0,
		/** Allow the file to be deleted or renamed while keeping this handle valid for reading. */
		AllowDelete = 1 << 1,
		/** 
		* Disables buffering performed by the platform operating system, if any. Note, this typically requires read 
		* offsets, and read sizes to be multiples of the volume's sector size, as well as the buffer being read into 
		* being aligned to a multiple of the physical sector size. Avoid using if you cannot control these parameters.
		*/
		NoBuffering = 1 << 2
	};

	/**
	 * Open a file handle for reading.
	 * 
	 * @param Filename	The file to be opened
	 * @param Flags		Allows specialization of the open operation, @see EOpenReadFlags
	 * 
	 * @return	The return value will either contain the valid file handle or an error message. @see FFileOpenResult
	 */
	CORE_API virtual FFileOpenResult OpenRead(const TCHAR* Filename, EOpenReadFlags Flags);

	/**
	 * Attempt to open a file for reading.
	 * Please consider using the new overload that takes EReadFlags instead of bools as parameters.
	 *
	 * @param Filename file to be opened
	 * @param bAllowWrite (applies to certain platforms only) whether this file is allowed to be written to by other processes. This flag is needed to open files that are currently being written to as well.
	 *
	 * @return If successful will return a non-nullptr pointer. Close the file by deleting the handle.
	 */
	virtual IFileHandle*	OpenRead(const TCHAR* Filename, bool bAllowWrite = false) = 0;

	virtual FFileOpenResult OpenReadNoBuffering(const TCHAR* Filename, EOpenReadFlags Flags)
	{
		return OpenRead(Filename, Flags);
	}

	virtual IFileHandle* OpenReadNoBuffering(const TCHAR* Filename, bool bAllowWrite = false)
	{
		return OpenRead(Filename, bAllowWrite);
	}

	/** Flags to be used when opening a file for writing. */
	enum class EOpenWriteFlags : uint8
	{
		/** No additional flags. */
		None		= 0,
		/** Allow the returned handle to be read from. On certain platforms allow other handles/processes to read from this file as well. */
		AllowRead	= 1 << 0,
		/** Append to file. */ 
		Append		= 1 << 1,
	};

	/** 
	 * Attempt to open a file for writing.
	 * NOTE: Physical IPlatformFile Implementations should override exactly one overload of OpenWrite, preferrably the version with flags.
	 * This overload will be removed in a future revision.
	 * This default implementation calls the Flags overload.
	 * 
	 * @param Filename		The file to be opened
	 * @param bAppend		Append to file.
	 * @param bAllowRead	Allow the returned handle to be read from. On certain platforms allow other handles/processes to read from this file as well.
	 * 
	 * @return	If successful will return a non-nullptr pointer. Close the file by deleting the handle. 
	 **/
	CORE_API virtual IFileHandle* OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false);
	/**
	 * Open a file handle for writing.
	 * NOTE: Physical IPlatformFile Implementations should override exactly one overload of OpenWrite, preferrably this one.
	 * This default implementation calls the Bools overload.
	 * 
	 * @param Filename	The file to be opened
	 * @param Flags		Allows specialization of the open operation, @see EOpenWriteFlags
	 * 
	 * @return	The return value will either contain the valid file handle or an error message. @see FFileOpenResult
	 */
	CORE_API virtual FFileOpenResult OpenWrite(const TCHAR* Filename, EOpenWriteFlags Flags);

	/** Return true if the directory exists. **/
	virtual bool		DirectoryExists(const TCHAR* Directory) = 0;
	/** Create a directory and return true if the directory was created or already existed. **/
	virtual bool		CreateDirectory(const TCHAR* Directory) = 0;
	/** Delete a directory and return true if the directory was deleted or otherwise does not exist. **/
	virtual bool		DeleteDirectory(const TCHAR* Directory) = 0;

	/** Return the stat data for the given file or directory. Check the FFileStatData::bIsValid member before using the returned data */
	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) = 0;

	/** Base class for file and directory visitors that take only the name. **/
	class FDirectoryVisitor
	{
	public:
		FDirectoryVisitor(EDirectoryVisitorFlags InDirectoryVisitorFlags = EDirectoryVisitorFlags::None)
			: DirectoryVisitorFlags(InDirectoryVisitorFlags)
		{
		}

		virtual ~FDirectoryVisitor() = default;

		/**
		 * Called with the LeafPathname (FullPath == Path/LeafPathname, LeafPathname == BaseName.Extension) before
		 * calling Visit. If it returns true, Visit will be called on the path, otherwise Visit will be skipped, and
		 * the return value of Visit is treated as true (continue iterating). Called both for directories and files.
		 */
		virtual bool ShouldVisitLeafPathname(FStringView LeafPathname)
		{
			return true;
		}

		/** 
		 * Callback for a single file or a directory in a directory iteration.
		 * @param FilenameOrDirectory		If bIsDirectory is true, this is a directory (with no trailing path delimiter), otherwise it is a file name.
		 * @param bIsDirectory				true if FilenameOrDirectory is a directory.
		 * @return							true if the iteration should continue.
		**/
		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) = 0;

		/**
		 * Helper function for receivers of FDirectoryVisitor. Enforces the contract for whether Visit should be
		 * called after calling ShouldVisitLeafPathName.
		 */
		CORE_API bool CallShouldVisitAndVisit(const TCHAR* FilenameOrDirectory, bool bIsDirectory);

		/** True if the Visit function can be called from multiple threads at once. **/
		UE_FORCEINLINE_HINT bool IsThreadSafe() const
		{
			return (DirectoryVisitorFlags & EDirectoryVisitorFlags::ThreadSafe) != EDirectoryVisitorFlags::None;
		}

		EDirectoryVisitorFlags DirectoryVisitorFlags;
	};

	/** File and directory visitor function that takes only the name */
	typedef TFunctionRef<bool(const TCHAR*, bool)> FDirectoryVisitorFunc;

	/** Base class for file and directory visitors that take all the stat data. **/
	class FDirectoryStatVisitor
	{
	public:
		virtual ~FDirectoryStatVisitor() = default;

		/**
		 * Called with the LeafPathname (FullPath == Path/LeafPathname, LeafPathname == BaseName.Extension) before
		 * calling Visit. If it returns true, Visit will be called on the path, otherwise Visit will be skipped, and
		 * the return value of Visit is treated as true (continue iterating). Called both for directories and files.
		 */
		virtual bool ShouldVisitLeafPathname(FStringView LeafPathname)
		{
			return true;
		}

		/**
		 * Callback for a single file or a directory in a directory iteration.
		 * @param FilenameOrDirectory		If bIsDirectory is true, this is a directory (with no trailing path delimiter), otherwise it is a file name.
		 * @param StatData					The stat data for the file or directory.
		 * @return							true if the iteration should continue.
		**/
		virtual bool Visit(const TCHAR* FilenameOrDirectory, const FFileStatData& StatData) = 0;

		/**
		 * Helper function for receivers of FDirectoryStatVisitor. Enforces the contract for whether Visit should be
		 *  called after calling ShouldVisitLeafPathName.
		 */
		CORE_API bool CallShouldVisitAndVisit(const TCHAR* FilenameOrDirectory, const FFileStatData& StatData);
	};

	/** File and directory visitor function that takes all the stat data */
	typedef TFunctionRef<bool(const TCHAR*, const FFileStatData&)> FDirectoryStatVisitorFunc;

	/** 
	 * Call the Visit function of the visitor once for each file or directory in a single directory. This function does not explore subdirectories.
	 * @param Directory		The directory to iterate the contents of.
	 * @param Visitor		Visitor to call for each element of the directory
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	virtual bool		IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor) = 0;

	/** 
	 * Call the Visit function of the visitor once for each file or directory in a single directory. This function does not explore subdirectories.
	 * @param Directory		The directory to iterate the contents of.
	 * @param Visitor		Visitor to call for each element of the directory
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	virtual bool		IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor) = 0;

	/**
	 * Return whether FileJournal functionality is available on the current platform if VolumeName is nullptr or for
	 * the given Volume if VolumeName is non-null. Optionally returns a user-displayable string for why it is not
	 * available and a severity level for the reason. VolumeName may be a VolumeName as returned by
	 * FileJournalGetVolumeName or any path to a file or directory on the Volume.
	 */
	CORE_API virtual bool FileJournalIsAvailable(const TCHAR* VolumeOrPath = nullptr,
		ELogVerbosity::Type* OutErrorLevel = nullptr, FString* OutError = nullptr);

	/**
	* Return the maximum size of the FileJournal for the specified VolumeName, if available, zero otherwise. Optionally returns
	* a user-displayable string for why the journal is not available and a severity level for the reason. VolumeName 
	* may be a VolumeName as returned by FileJournalGetVolumeName or any path to a file or directory on the Volume.
	*/
	CORE_API virtual uint64 FileJournalGetMaximumSize(const TCHAR* VolumeOrPath = nullptr,
		ELogVerbosity::Type* OutErrorLevel = nullptr, FString* OutError = nullptr) const;

	/**
	 * Report the current end of the journal for the given volume, to be used as the StartingJournalEntry
	 * in FileJournalGetModifiedDirectories. If !FileJournalIsAvaiable for the given volume, sets OutEntryHandle
	 * to FileJournalEntryHandleInvalid. Returns EFileJournalResult::Success if successful, otherwise an error code
	 * and optionally a user-displayable explanation for the error code.
	 */
	CORE_API virtual EFileJournalResult FileJournalGetLatestEntry(const TCHAR* VolumeName,
		FFileJournalId& OutJournalId, FFileJournalEntryHandle& OutEntryHandle, FString* OutError = nullptr);

	/** File and directory visitor function that takes FileJournal data. */
	typedef TFunctionRef<bool(const TCHAR*, const FFileJournalData&)> FDirectoryJournalVisitorFunc;

	/**
	 * Iterate the given directory as with IterateDirectoryStat, but report a FFileJournalData for each 
	 * file and directory, which notably includes the FFileJournalFileHandle for the file/directory.
	 * 
	 * The paths returned as the first argument of the visitor function are the combined paths produced
	 * by combining the input directory with the relative path of the child file or directory.
	 * 
	 * If the FileJournal is unavailable on the current system the iteration will still succeed but the 
	 * FFileJournalFileHandle for each child path will be set to FileJournalFileHandleInvalid.
	 * 
	 * If the FileJournal is available on the current system but not on the volume of the given directory,
	 * it is arbitrary whether the FFileJournalFileHandle will be validly set; if not valid they will be
	 * set to FileJournalFileHandleInvalid.
	 * 
	 * @return	false if the directory did not exist or if the visitor returned false.
	 */
	CORE_API virtual bool FileJournalIterateDirectory(const TCHAR* Directory, FDirectoryJournalVisitorFunc Visitor, FString* OutError = nullptr);

	/**
	 * Return the data for the given path as with GetStatData, but report a FFileJournalData instead, which
	 * notably includes the FFileJournalFileHandle for the file/directory.
	 * Check the FFileJournalData::bIsValid member before using the returned data 
	 */
	CORE_API virtual FFileJournalData FileJournalGetFileData(const TCHAR* FilenameOrDirectory, FString* OutError = nullptr);

	/**
	 * Query the FileJournal to find a list of all directories on the given volume with files that have been added,
	 * deleted, or modified in the specified time range. The beginning of the time range is specified by
	 * JournalIdOfStartingEntry and StartingJournalEntry, which came from FileJournalGetLatestEntry or a previous
	 * call to FileJournalReadModified. The end of the range is the latest modification on the volume. VolumeName can
	 * be the return value from FileJournalGetVolumeName, or any path on the desired volume.
	 * 
	 * The caller must provide the mapping from FFileJournalFileHandle to DirectoryName; the FFileJournalFileHandle 
	 * for each Directory can be found from FileJournalGetFileData or FileJournalIterateDirectory.
	 *
	 * Modified directories are appended into OutModifiedDirectories, and the next FileJournal entry to scan is written
	 * into OutNextJournalEntry.
	 * 
	 * Returns EFileJournalResult::Success if successful, otherwise an error code and optionally a user-displayable
	 * explanation for the error code. In an error case, partial results may still be written into the output.
	 */
	CORE_API virtual EFileJournalResult FileJournalReadModified(const TCHAR* VolumeName,
		const FFileJournalId& JournalIdOfStartingEntry, const FFileJournalEntryHandle& StartingJournalEntry,
		TMap<FFileJournalFileHandle, FString>& KnownDirectories, TSet<FString>& OutModifiedDirectories,
		FFileJournalEntryHandle& OutNextJournalEntry, FString* OutError = nullptr);

	/**
	 * Return the VolumeSpecifier present in the given path. Returns empty string if path does not have a valid
	 * volume specifier for use by the FileJournal (e.g. some platforms do not support \\paths for FileJournal).
	 * @see FPathViews::SplitVolumeSpecifier for a more general function that can return specifiers not usable
	 * by the FileJournal.
	 */
	CORE_API virtual FString FileJournalGetVolumeName(FStringView InPath);

	/////////////////////////////////////////////////////////////////////////////////////////////////////////
	/////////// Utility Functions. These have a default implementation that uses the pure virtual operations.
	/////////// Generally, these do not need to be implemented per platform.
	/////////////////////////////////////////////////////////////////////////////////////////////////////////

	/** Open a file for async reading. This call does not hit the disk or block.
	* 
	* @param Flags		Allows specialization of the open operation, @see EReadFlags
	* @return	The return value will either contain the valid file handle or an error message. @see FFileOpenAsyncResult
	*/
	CORE_API virtual FFileOpenAsyncResult OpenAsyncRead(const TCHAR* Filename, EOpenReadFlags Flags);

	/** Open a file for async reading. This call does not hit the disk or block.
	*
	* @param Filename file to be opened
	* @param bAllowWrite (applies to certain platforms only) whether this file is allowed to be written to by other processes. This flag is needed to open files that are currently being written to as well.
	* @return Close the file by delete'ing the handle. A non-null return value does not mean the file exists, since that may not be determined yet.
	*/
	CORE_API virtual IAsyncReadFileHandle* OpenAsyncRead(const TCHAR* Filename, bool bAllowWrite = false);

	/** Controls if the pak precacher should process precache requests.
	* Requests below this threshold will not get precached. Without this throttle, quite a lot of memory
	* can be consumed if the disk races ahead of the CPU.
	* @param MinPriority the minimum priority at which requests will get precached
	*/
	virtual void SetAsyncMinimumPriority(EAsyncIOPriorityAndFlags MinPriority)
	{
	}

	/** Flags to be used when opening a file for via IPlatformFile::OpenMappedEx */
	enum class EOpenMappedFlags : uint8
	{
		None			= 0,
		/** Allow the handle to be written to via the memory map. */
		OpenForWrite	= 1 << 0,
		/** If the file already exists, do not replace it. Only applies if OpenForWrite is set. */
		OpenExisting	= 1 << 1,
		/** Allow other handles/processes to write to this file. 
		  * On certain platforms this flag is needed to open files that are currently being written to as well. 
		  * Only applies if OpenForWrite is not set.
		  */
		AllowWrite		= 1 << 2,
		/** On certain platforms allow other handles/processes to read from this file as well.
		  * Only applies if OpenForWrite is set.
		  */
		AllowRead		= 1 << 3,
		/** Allow the file to be deleted or renamed while keeping this handle valid for reading */
		AllowDelete		= 1 << 4,
		
	};

	/** Open a file for async reading. This call does hit the disk; it is synchronous open.
	*
	* @param Filename		File to be mapped. This doesn't actually map anything, just opens the file.
	* @param OpenOptions	Allows specifying map to be writable via AllowWrite or be ReadOnly by default.
	* @return Close the file by deleting the handle. A non-null return value does mean the file exists.
	* Null can be returned for many reasons even if the file exists. Perhaps this platform does not support mapped files, or this file is compressed in a pak file.
	* Generally you attempt to open mapped, and if that fails, then use other file operations instead.
	* The exact file open behavior may vary per-platform when using AllowWrite.
	* Prefer OpenMappedEx2 which has more consistent behavior across all supported platforms. 
	* OpenMappedEx is not currently deprecated so as not to change any platform specific behavior that may be implicitly relied upon.
	*/
	CORE_API virtual FOpenMappedResult OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions = EOpenReadFlags::None, int64 MaximumSize = 0);

	/** Open a file for async reading. This call does hit the disk; it is synchronous open.
	*
	* @param Filename		File to be mapped. This doesn't actually map anything, just opens the file.
	* @param OpenOptions	Allows specifying map to be writable via OpenForWrite or be ReadOnly by default.
	* @return Close the file by deleting the handle. A non-null return value does mean the file exists.
	* Null can be returned for many reasons even if the file exists. Perhaps this platform does not support mapped files, or this file is compressed in a pak file.
	* Generally you attempt to open mapped, and if that fails, then use other file operations instead.
	* Prefer OpenMappedEx2 which has consistent behavior across all supported platforms.
	*/
	CORE_API virtual FOpenMappedResult OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions = EOpenMappedFlags::None, int64 MaximumSize = 0);

	CORE_API virtual void GetTimeStampPair(const TCHAR* PathA, const TCHAR* PathB, FDateTime& OutTimeStampA, FDateTime& OutTimeStampB);

	/** Return the modification time of a file in the local time of the calling code (GetTimeStamp returns UTC). Returns FDateTime::MinValue() on failure **/
	CORE_API virtual FDateTime	GetTimeStampLocal(const TCHAR* Filename);

	/**
	 * Call the visitor once for each file or directory in a single directory. This function does not explore subdirectories.
	 * @param Directory		The directory to iterate the contents of.
	 * @param Visitor		Visitor to call for each element of the directory (see FDirectoryVisitor::Visit for the signature)
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	CORE_API virtual bool IterateDirectory(const TCHAR* Directory, FDirectoryVisitorFunc Visitor);

	/**
	 * Call the visitor once for each file or directory in a single directory. This function does not explore subdirectories.
	 * @param Directory		The directory to iterate the contents of.
	 * @param Visitor		Visitor to call for each element of the directory (see FDirectoryStatVisitor::Visit for the signature)
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	CORE_API virtual bool IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitorFunc Visitor);

	/** 
	 * Call the Visit function of the visitor once for each file or directory in a directory tree. This function explores subdirectories.
	 * @param Directory		The directory to iterate the contents of, recursively.
	 * @param Visitor		Visitor to call for each element of the directory and each element of all subdirectories.
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	CORE_API virtual bool IterateDirectoryRecursively(const TCHAR* Directory, FDirectoryVisitor& Visitor);

	/** 
	 * Call the Visit function of the visitor once for each file or directory in a directory tree. This function explores subdirectories.
	 * @param Directory		The directory to iterate the contents of, recursively.
	 * @param Visitor		Visitor to call for each element of the directory and each element of all subdirectories.
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	CORE_API virtual bool IterateDirectoryStatRecursively(const TCHAR* Directory, FDirectoryStatVisitor& Visitor);

	/**
	 * Call the Visit function of the visitor once for each file or directory in a directory tree. This function explores subdirectories.
	 * @param Directory		The directory to iterate the contents of, recursively.
	 * @param Visitor		Visitor to call for each element of the directory and each element of all subdirectories (see FDirectoryVisitor::Visit for the signature).
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	CORE_API virtual bool IterateDirectoryRecursively(const TCHAR* Directory, FDirectoryVisitorFunc Visitor);

	/**
	 * Call the Visit function of the visitor once for each file or directory in a directory tree. This function explores subdirectories.
	 * @param Directory		The directory to iterate the contents of, recursively.
	 * @param Visitor		Visitor to call for each element of the directory and each element of all subdirectories (see FDirectoryStatVisitor::Visit for the signature).
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	CORE_API virtual bool IterateDirectoryStatRecursively(const TCHAR* Directory, FDirectoryStatVisitorFunc Visitor);
		
	/**
	 * Finds all the files within the given directory, with optional file extension filter
	 * @param Directory			The directory to iterate the contents of
	 * @param FileExtension		If FileExtension is NULL, or an empty string "" then all files are found.
	 * 							Otherwise FileExtension can be of the form .EXT or just EXT and only files with that extension will be returned.
	 * @return FoundFiles		All the files that matched the optional FileExtension filter, or all files if none was specified.
	 */
	CORE_API virtual void FindFiles(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension);

	/**
	 * Finds all the files within the directory tree, with optional file extension filter
	 * @param Directory			The starting directory to iterate the contents. This function explores subdirectories
	 * @param FileExtension		If FileExtension is NULL, or an empty string "" then all files are found.
	 * 							Otherwise FileExtension can be of the form .EXT or just EXT and only files with that extension will be returned.
	 * @return FoundFiles		All the files that matched the optional FileExtension filter, or all files if none was specified.
	 */
	CORE_API virtual void FindFilesRecursively(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension);

	/** 
	 * Delete all files and subdirectories in a directory, then delete the directory itself
	 * @param Directory		The directory to delete.
	 * @return				true if the directory was deleted or did not exist.
	**/
	CORE_API virtual bool DeleteDirectoryRecursively(const TCHAR* Directory);

	/** Create a directory, including any parent directories and return true if the directory was created or already existed. **/
	CORE_API virtual bool CreateDirectoryTree(const TCHAR* Directory);

	/** 
	 * Copy a file. This will fail if the destination file already exists.
	 * @param To				File to copy to.
	 * @param From				File to copy from.
	 * @param ReadFlags			Source file read options.
	 * @param WriteFlags		Destination file write options.
	 * @return					true if the file was copied successfully.
	**/
	CORE_API virtual bool CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags = EPlatformFileRead::None, EPlatformFileWrite WriteFlags = EPlatformFileWrite::None);

	/** 
	 * Copy a file or a hierarchy of files (directory).
	 * @param DestinationDirectory			Target path (either absolute or relative) to copy to - always a directory! (e.g. "/home/dest/").
	 * @param Source						Source file (or directory) to copy (e.g. "/home/source/stuff").
	 * @param bOverwriteAllExisting			Whether to overwrite everything that exists at target
	 * @return								true if operation completed successfully.
	 */
	CORE_API virtual bool CopyDirectoryTree(const TCHAR* DestinationDirectory, const TCHAR* Source, bool bOverwriteAllExisting);

	/**
	 * Converts passed in filename to use an absolute path (for reading).
	 *
	 * @param	Filename	filename to convert to use an absolute path, safe to pass in already using absolute path
	 * 
	 * @return	filename using absolute path
	 */
	CORE_API virtual FString ConvertToAbsolutePathForExternalAppForRead( const TCHAR* Filename );

	/**
	 * Converts passed in filename to use an absolute path (for writing)
	 *
	 * @param	Filename	filename to convert to use an absolute path, safe to pass in already using absolute path
	 * 
	 * @return	filename using absolute path
	 */
	CORE_API virtual FString ConvertToAbsolutePathForExternalAppForWrite( const TCHAR* Filename );

	/**
	 * Helper class to send/receive data to the file server function
	 */
	class IFileServerMessageHandler
	{
	public:
		virtual ~IFileServerMessageHandler() = default;

		/** Subclass fills out an archive to send to the server */
		virtual void FillPayload(FArchive& Payload) = 0;

		/** Subclass pulls data response from the server */
		virtual void ProcessResponse(FArchive& Response) = 0;
	};

	/**
	 * Sends a message to the file server, and will block until it's complete. Will return 
	 * immediately if the file manager doesn't support talking to a server.
	 *
	 * @param Message	The string message to send to the server
	 *
	 * @return			true if the message was sent to server and it returned success, or false if there is no server, or the command failed
	 */
	virtual bool SendMessageToServer(const TCHAR* Message, IFileServerMessageHandler* Handler)
	{
		// by default, IPlatformFile's can't talk to a server
		return false;
	}
	
	/**
	 * Checks to see if this file system creates publicly accessible files
	 *
	 * @return			true if this file system creates publicly accessible files
	 */
	virtual bool DoesCreatePublicFiles()
	{
		return false;
	}
	
	/**
	 * Sets file system to create publicly accessible files or not
	 *
	 * @param bCreatePublicFiles			true to set the file system to create publicly accessible files
	 */
	virtual void SetCreatePublicFiles(bool bCreatePublicFiles)
	{
	}

	/**
	 * Returns the number of bytes that are currently allowed to be written to throttled write storage (if the platform 
	 * has such restrictions)
	 * 
	 * @param DestinationPath		If specified, the file system can optionally take into account the destination of 
	 *								the file to determine the current limit
	 * 
	 * @returns						The number of bytes that are allowed to be written to write throttled storage.  
	 *								If there is no limit, INT64_MAX is returned
	 */
	virtual int64 GetAllowedBytesToWriteThrottledStorage(const TCHAR* DestinationPath = nullptr)
	{
		return INT64_MAX;
	}
};

class IWrappedFileHandle : public IFileHandle
{
public:
	virtual void SetLowerLevel(IFileHandle* LowerLevelHandle) = 0;
};

/**
* Common base for physical platform File I/O Interface
**/
class IPhysicalPlatformFile : public IPlatformFile
{
public:
	//~ Begin IPlatformFile Interface
	virtual bool ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const override
	{
		return true;
	}
	CORE_API virtual bool Initialize(IPlatformFile* Inner, const TCHAR* CmdLine) override;
	virtual IPlatformFile* GetLowerLevel() override
	{
		return nullptr;
	}
	virtual void SetLowerLevel(IPlatformFile* NewLowerLevel) override
	{
		check(false); // can't override wrapped platform file for physical platform file
	}
	virtual const TCHAR* GetName() const override
	{
		return IPlatformFile::GetPhysicalTypeName();
	}
	//~ End IPlatformFile Interface
};

/* Interface class for FPakFile to allow usage from modules that cannot have a compile dependency on FPakFile. See FCoreDelegates */
class IPakFile
{
public:
	virtual const FString& PakGetPakFilename() const = 0;
	/**
	  * Return whether the Pak has an entry for the given FileName.  Not necessarily exclusive; other Patch Paks may have their own copy of the same File.
	  * @param Filename The full LongPackageName path to the file, as returned from FPackageName::LongPackageNameToFilename + extension.  Comparison is case-insensitive.
	  */
	virtual bool PakContains(const FString& Filename) const = 0;
	virtual int32 PakGetPakchunkIndex() const = 0;
	/**
	 * Calls the given Visitor on every FileName in the Pruned Directory Index. FileNames passed to the Vistory are the RelativePath from the Mount of the PakFile
	 * The Pruned Directory Index at Runtime contains only the DirectoryIndexKeepFiles-specified subset of FilesNames and DirectoryNames that exist in the PakFile
	 */
	virtual void PakVisitPrunedFilenames(IPlatformFile::FDirectoryVisitor& Visitor) const = 0;
	virtual const FString& PakGetMountPoint() const = 0;

	virtual int32 GetNumFiles() const = 0;
};

/* Pak Mount Options allow usage from modules that cannot have a compile dependency on FPakFile. See FCoreDelegates. */
struct FPakMountOptions
{
	enum class EMountFlags : uint32
	{
		None					= 0,
		WithSoftReferences		= 1 << 0,
		ReportDecryptionFailure = 1 << 1,
		SkipContainerFile		= 1 << 2,
	};

	EMountFlags MountFlags = EMountFlags::None;
};
ENUM_CLASS_FLAGS(FPakMountOptions::EMountFlags);

inline uint32 GetTypeHash(const FFileJournalFileHandle& A)
{
	constexpr uint32 Mult = 103;
	uint32 Hash = 0;
	const uint8* EndA = reinterpret_cast<const uint8*>(&A) + sizeof(FFileJournalFileHandle);
	for (const uint8* PA = reinterpret_cast<const uint8*>(&A); PA < EndA; ++PA)
	{
		Hash = Hash * Mult + *PA;
	}
	return Hash;
}

inline bool FFileJournalFileHandle::operator==(const FFileJournalFileHandle& Other) const
{
	return 0 == FPlatformMemory::Memcmp(this, &Other, sizeof(FFileJournalFileHandle));
}

inline bool FFileJournalFileHandle::operator!=(const FFileJournalFileHandle& Other) const
{
	return 0 != FPlatformMemory::Memcmp(this, &Other, sizeof(FFileJournalFileHandle));
}

inline FFileJournalData::FFileJournalData()
	: bIsValid(false)
	, bIsDirectory(false)
	, bIsReparsePoint(false)
	, bIsReadOnly(false)
{
}

ENUM_CLASS_FLAGS(IPlatformFile::EOpenReadFlags);
ENUM_CLASS_FLAGS(IPlatformFile::EOpenWriteFlags);
ENUM_CLASS_FLAGS(IPlatformFile::EOpenMappedFlags);
