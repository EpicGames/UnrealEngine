// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformHostSocket.h"



/**
 * Known custom communication protocols used by the engine.
 * 
 * These values may be used as ProtocolIndex parameter when calling OpenConnection.
 * 
 * @see IPlatformHostCommunication::OpenConnection
 */
enum EHostProtocol
{
	CookOnTheFly = 0
};


/**
 * Interface for communication between the game running on the target device and the connected host pc.
 * 
 * It represents a custom communication channel and may not be implemented on all platforms.
 *
 * It is meant to be used in development ONLY.
 */
class IPlatformHostCommunication
{
public:

	/**
	  * Initialize the host communication system (done by IPlatformFeaturesModule).
	  */
	virtual void Initialize() = 0;

	/**
	 * Shutdown the host communication system (done by IPlatformFeaturesModule).
	 */
	virtual void Shutdown() = 0;

	/**
	 * Check if the host communication system is available and can be used.
	 * 
	 * @return True is available, false otherwise.
	 */
	virtual bool Available() const = 0;

	/**
	 * Open a communication channel with the host PC.
	 * 
	 * The connected PC is determined in a platform-dependent way.
	 * 
	 * Note that opening a connection does not necessarily mean that the host PC is ready to communicate.
	 * Check the state of the socket before attempting to send/receive data. There can be only one
	 * connection using any given ProtocolIndex. Attempts to use a ProtocolIndex while there is already
	 * a connection using it will fail.
	 * 
	 * @param ProtocolIndex Arbitrary 0-based number indicating the communication channel (the same number needs to be used on the host PC).
	 * @param DebugName     Name of this communication channel used for diagnostic purposes.
	 * @param Version       (optional) Version of the communication protocol (ProtocolIndex) used to verify compatibility with the host PC.
	 * @param MinVersion    (optional) Minimum version of the communication protocol (ProtocolIndex) that is supported.
	 * @return              Socket object on success, nullptr otherwise.
	 * 
	 * @see CloseConnection, IPlatformHostSocket::GetState
	 */
	virtual IPlatformHostSocketPtr OpenConnection(uint32 ProtocolIndex, const FString& DebugName, uint32 Version = 0, uint32 MinVersion = 0) = 0;

	/**
	 * Close a communication channel previously opened using OpenConnection.
	 * @param Socket Socket object.
	 */
	virtual void CloseConnection(IPlatformHostSocketPtr Socket) = 0;

	/**
	 * Launch an executable on the connected host PC.
	 * @param BinaryPath Path to the executable on the PC to launch.
	 * @param CmdLine    (optional) Command line parameters to pass to the executable.
	 * @return           True on success, otherwise false (e.g. no permissions or incorrect executable path).
	 */
	virtual bool LaunchOnHost(const char* BinaryPath, const char* CmdLine = nullptr) = 0;

	/**
	 * Virtual destructor.
	 */
	virtual ~IPlatformHostCommunication() = default;
};


/**
 * Generic implementation of IPlatformHostCommunication.
 */
class FGenericPlatformHostCommunication : public IPlatformHostCommunication
{
public:

	virtual void Initialize() override
	{
	}

	virtual void Shutdown() override
	{
	}

	virtual bool Available() const override
	{
		return false;
	}

	virtual IPlatformHostSocketPtr OpenConnection(uint32 ProtocolIndex, const FString& DebugName, uint32 Version = 0, uint32 MinVersion = 0) override
	{
		return nullptr;
	}

	void CloseConnection(IPlatformHostSocketPtr Socket) override
	{
	}

	bool LaunchOnHost(const char* BinaryPath, const char* CmdLine = nullptr) override
	{
		return false;
	}
};


/**
 * Helper RAII type for auto initializing instances derived from IPlatformHostCommunication.
 */
template <typename T>
struct FPlatformHostCommunicationAutoInit
{
	FPlatformHostCommunicationAutoInit()
	{
		HostCommunication.Initialize();
	}

	~FPlatformHostCommunicationAutoInit()
	{
		HostCommunication.Shutdown();
	}

	operator T& ()
	{
		return HostCommunication;
	}

	T HostCommunication;
};
