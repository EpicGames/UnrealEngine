// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"



/**
 * Interface for sockets supporting direct communication between the game running
 * on the target device and a connected PC.
 * 
 * It represents a custom communication channel and may not be implemented on all platforms.
 * 
 * It is meant to be used in development ONLY.
 * 
 * @see IPlatformHostCommunication
 */
class IPlatformHostSocket
{
public:

	/**
	  * Status values returned by Send and Receive members.
	  * @see Send, Receive
	  */
	enum class EResultNet : uint8
	{
		Ok,							// Communication successful.
		ErrorUnknown,				// Unknown error.
		ErrorInvalidArgument,		// Incorrect parameters provided to a function (shouldn't happen assuming the socket object is valid).
		ErrorInvalidConnection,		// Incorrect socket id used (shouldn't happen assuming the socket object is valid).
		ErrorInterrupted,			// Data transfer interrupted e.g. a networking issue.
		ErrorHostNotConnected		// Host PC is not connected (not connected yet or has already disconnected).
	};

	/**
	  * State of the socket determining its ability to send/receive data.
	  * @see GetState
	  */
	enum class EConnectionState : uint8
	{
		Unknown,		// Default state (shouldn't be returned).
		Created,		// Socket has been created by cannot communicate yet (no host pc connected yet).
		Connected,		// Socket ready for communication.
		Disconnected,	// Host PC has disconnected (no communication possible, socket should be closed).
		Closed,			// Socket has already been closed and shouldn't be used.
	};

	/**
	 * Mode of a socket read
	 */
	enum class EReceiveFlags : uint8
	{
		DontWait,		// Read as much there is on the wire up to the buffer size
		WaitAll,		// Block read and wait until the buffer is filled
	};

public:

	/**
	 * Send data to the connected host PC (blocking operation).
	 * 
	 * @param Buffer      Data to be sent.
	 * @param BytesToSend The number of bytes to send.
	 * @return            Status value indicating error or success.
	 */
	virtual EResultNet Send(const void* Buffer, uint64 BytesToSend) = 0;

	/**
	 * Receive data from the connected host PC (blocking operation).
	 *
	 * @param Buffer         Data to be sent.
	 * @param BytesToReceive The number of bytes to receive (Buffer has to be large enough).
	 * @return               Status value indicating error or success.
	 */
	virtual EResultNet Receive(void* Buffer, uint64 BytesToReceive) = 0;

	/**
	 * Receive data from the connected host PC.
	 * 
	 * @param Buffer         Data to be sent.
	 * @param BytesToReceive The number of bytes to receive (Buffer has to be large enough).
	 * @param BytesReceived  Number of bytes that have been received (equals to BytesToReceive if ReadMode is EReceiveFlags::WaitAll)
	 * @param ReadMode       DontWait if this call should return immediately with the data available and not wait for BytesToReceive number of bytes
	 * @return               Status value indicating error or success.
	 */
	virtual EResultNet Receive(void* Buffer, uint64 BytesToReceive, uint64& BytesReceived, EReceiveFlags ReadMode = EReceiveFlags::WaitAll) = 0;

	/**
	 * Get the state of the socket (determines if the host pc is connected and communication is possible).
	 */
	virtual EConnectionState GetState() const = 0;

	/**
	 * Destructor.
	 */
	virtual ~IPlatformHostSocket() = default;
};


// Type definition for shared references to instances of IPlatformHostSocket.
typedef TSharedRef<IPlatformHostSocket, ESPMode::ThreadSafe> IPlatformHostSocketRef;

// Type definition for shared pointers to instances of IPlatformHostSocket.
typedef TSharedPtr<IPlatformHostSocket, ESPMode::ThreadSafe> IPlatformHostSocketPtr;
