// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/MemoryBase.h"

struct FMallocCrashPool;
struct FPoolDesc;

/** Describes a pointer. */
struct FPtrInfo
{
	/** Size of the allocation. */
	uint64 Size;
	/** Address of the allocation. */
	uint8* Ptr;

	FPtrInfo():
		Size(0),
		Ptr(0)
	{}

	FPtrInfo( void* NewPtr ):
		Size(0),
		Ptr((uint8*)NewPtr)
	{}
};

struct FMallocCrashPool;
struct FPoolDesc;

/**
 * Simple pooled memory allocator that uses preallocated memory.
 * Instance of this class replaces GMalloc after a crash, so we can use dynamic memory allocation even if the app crashes due to OOM.
 */
struct FGenericPlatformMallocCrash final : public FMalloc
{
	friend struct FPoolDesc;
	friend struct FMallocCrashPool;

	enum
	{
		LARGE_MEMORYPOOL_SIZE = 2 * 1024 * 1024,
		REQUIRED_ALIGNMENT = 16,

		NUM_POOLS = 14,
		MAX_NUM_ALLOCS_IN_POOL = 2048,

		MEM_TAG     = 0xfe,
		MEM_WIPETAG = 0xcd,
	};

public:
	FGenericPlatformMallocCrash( FMalloc* MainMalloc );
	virtual ~FGenericPlatformMallocCrash();

	/** Creates a new instance. */
	static CORE_API FGenericPlatformMallocCrash& Get( FMalloc* MainMalloc = nullptr );

	/**
	 * Sets as GMalloc.
	 * This method locks to the thread that crashed.
	 * Any next calls to this method or GMalloc from other threads will dead-lock those threads,
	 * but this is ok, because we are shutting down.
	 * This also fixes a lot of potential issues of using dynamic allocation during crash dumping.
	 * 
	 * @warning
	 * This is not super safe, may interfere with other allocations, ie. replacing vtable during running 
	 * the code from the previous malloc will probably crash other threads.
	 */
	void SetAsGMalloc();

	static bool IsActive();

	// FMalloc interface.
	virtual void* Malloc( SIZE_T Size, uint32 Alignment ) override;

	virtual void* Realloc( void* Ptr, SIZE_T NewSize, uint32 Alignment ) override;

	virtual void Free( void* /*Ptr*/ ) override;

	virtual bool GetAllocationSize( void *Original, SIZE_T &SizeOut ) override;

	virtual bool IsInternallyThreadSafe() const override
	{
		return true;
	}

	virtual bool ValidateHeap() override
	{
		// Nothing to do here.
		return true;
	}

	virtual const TCHAR * GetDescriptiveName() override
	{
		return TEXT( "MallocCrash" );
	}

	CORE_API void PrintPoolsUsage();

protected:
	/** Whether it is safe to call crash malloc's methods. */
	bool CheckThreadForAllocation() const;
	bool CheckThreadForFree() const;

	bool IsPtrInLargePool(void* Ptr) const;
	bool IsPtrInSmallPool(void* Ptr) const;

	const FPoolDesc& GetPoolDesc( uint32 Index ) const;
	uint32 CalculateSmallPoolTotalSize() const;
	uint32 CalculateBookkeepingPoolTotalSize() const;

	void InitializeSmallPools();

	// Choose a pool to make a new allocation from of the given size - may use a larger pool than necessary if small pools are exhausted.
	FMallocCrashPool* ChoosePoolForSize( uint32 AllocationSize ) const;
	// Find the pool that an allocation was made from if any.
	FMallocCrashPool* FindPoolForAlloc(void* Ptr) const;
	uint8* AllocateFromBookkeeping(uint32 AllocationSize);
	uint8* AllocateFromSmallPool( uint32 AllocationSize );

	uint64 GetAllocationSize(void *Original);

	/**
	 * @return page size, if page size is not initialized, returns 64k.
	 */
	static uint32 SafePageSize();

protected:
	/** Used to lock crash malloc to one thread. */
	FCriticalSection InternalLock;

	/** The id of the thread that crashed. */
	TSAN_ATOMIC(int32) CrashedThreadId = 0;

	/** Preallocated memory pool for large allocations. */
	uint8* LargeMemoryPool = nullptr;

	/** Current position in the large memory pool. */
	uint32 LargeMemoryPoolOffset = 0;

	/** Preallocated memory pool for small allocations. */
	uint8* SmallMemoryPool = nullptr;

	/** Current position in the small memory pool. */
	uint32 SmallMemoryPoolOffset = 0;
	uint32 SmallMemoryPoolSize = 0;

	/** Preallocated memory pool for bookkeeping. */
	uint8* BookkeepingPool = nullptr;

	/** Current position in the bookkeeping pool. */
	uint32 BookkeepingPoolOffset = 0;
	uint32 BookkeepingPoolSize = 0;

	/** Previously used malloc. */
	FMalloc* PreviousMalloc = nullptr;
};





struct FGenericStackBasedMallocCrash : public FMalloc
{
	FGenericStackBasedMallocCrash(FMalloc* MainMalloc);
	virtual ~FGenericStackBasedMallocCrash();

	/** Creates a new instance. */
	static CORE_API FGenericStackBasedMallocCrash& Get(FMalloc* MainMalloc = nullptr);

	void SetAsGMalloc();

	static bool IsActive();

	virtual void* Malloc(SIZE_T Size, uint32 Alignment) override;

	virtual void* Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override;

	virtual void Free(void* /*Ptr*/) override;

	virtual const TCHAR * GetDescriptiveName() override
	{
		return TEXT("FGenericStackBasedMallocCrash");
	}

private:
	enum
	{
		MEMORYPOOL_SIZE = 256 * 1024
	};

	uint8* CurrentFreeMemPtr;
	uint8* FreeMemoryEndPtr;
};
