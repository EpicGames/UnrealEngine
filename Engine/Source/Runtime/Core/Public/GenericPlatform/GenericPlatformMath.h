// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	GenericPlatformMath.h: Generic platform Math classes, mostly implemented with ANSI C++
==============================================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Concepts/Integral.h"
#include "Containers/ContainersFwd.h"
#include "HAL/PlatformCrt.h"
#include "Templates/AndOrNot.h"
#include "Templates/Decay.h"
#include "Templates/IsFloatingPoint.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/Requires.h"
#include "Templates/ResolveTypeAmbiguity.h"
#include "Templates/TypeCompatibleBytes.h"
#include <limits>
#include <type_traits>

#if PLATFORM_HAS_FENV_H 
#include <fenv.h>
#endif 

/**
 * Generic implementation for most platforms
 */
struct FGenericPlatformMath
{
	// load half (F16) to float
	//https://gist.github.com/rygorous/2156668
	[[nodiscard]] static constexpr inline float LoadHalf(const uint16* Ptr)
	{
		uint16 FP16 = *Ptr;
		constexpr uint32 shifted_exp = 0x7c00 << 13;			// exponent mask after shift
		union FP32T
		{
			uint32 u;
			float f;		
		} FP32 = {}, magic = { 113 << 23 };

		FP32.u = (FP16 & 0x7fff) << 13;				// exponent/mantissa bits
		uint32 exp = shifted_exp & FP32.u;			// just the exponent
		FP32.u += uint32(127 - 15) << 23;			// exponent adjust

		// handle exponent special cases
		if (exp == shifted_exp)						// Inf/NaN?
		{
			FP32.u += uint32(128 - 16) << 23;		// extra exp adjust
		}
		else if (exp == 0)							// Zero/Denormal?
		{
			FP32.u += 1 << 23;						// extra exp adjust
			FP32.f -= magic.f;						// renormalize
		}

		FP32.u |= (FP16 & 0x8000) << 16;			// sign bit
		return FP32.f;
	}

	// store float to half (F16)
	// converts with RTNE = round to nearest even
	// values too large for F16 are stored as +-Inf
	// https://gist.github.com/rygorous/2156668
	// float_to_half_fast3_rtne
	static constexpr inline void StoreHalf(uint16* Ptr, float Value)
	{
		union FP32T
		{
			uint32 u;
			float f;
		} FP32 = {};
		uint16 FP16 = {};

		FP32.f = Value;

		constexpr FP32T f32infty = { uint32(255 << 23) };
		constexpr FP32T f16max = { uint32(127 + 16) << 23 };
		constexpr FP32T denorm_magic = { (uint32(127 - 15) + uint32(23 - 10) + 1) << 23 };
		constexpr uint32 sign_mask = 0x80000000u;

		uint32 sign = FP32.u & sign_mask;
		FP32.u ^= sign;

		// NOTE all the integer compares in this function can be safely
		// compiled into signed compares since all operands are below
		// 0x80000000. Important if you want fast straight SSE2 code
		// (since there's no unsigned PCMPGTD).

		if (FP32.u >= f16max.u) // result is Inf or NaN (all exponent bits set)
		{
			FP16 = (FP32.u > f32infty.u) ? 0x7e00 : 0x7c00; // NaN->qNaN and Inf->Inf
		}
		else // (De)normalized number or zero
		{
			if (FP32.u < uint32(113 << 23)) // resulting FP16 is subnormal or zero
			{
				// use a magic value to align our 10 mantissa bits at the bottom of
				// the float. as long as FP addition is round-to-nearest-even this
				// just works.
				FP32.f += denorm_magic.f;

				// and one integer subtract of the bias later, we have our final float!
				FP16 = uint16(FP32.u - denorm_magic.u);
			}
			else
			{
				uint32 mant_odd = (FP32.u >> 13) & 1; // resulting mantissa is odd

				// update exponent, rounding bias part 1
				FP32.u += (uint32(15 - 127) << 23) + 0xfff;
				// rounding bias part 2
				FP32.u += mant_odd;
				// take the bits!
				FP16 = uint16(FP32.u >> 13);
			}
		}

		FP16 |= sign >> 16;
		*Ptr = FP16;
	}

	static constexpr inline void VectorLoadHalf(float* RESTRICT Dst, const uint16* RESTRICT Src)
	{
		Dst[0] = LoadHalf(&Src[0]);
		Dst[1] = LoadHalf(&Src[1]);
		Dst[2] = LoadHalf(&Src[2]);
		Dst[3] = LoadHalf(&Src[3]);
	}

	static constexpr inline void VectorStoreHalf(uint16* RESTRICT Dst, const float* RESTRICT Src)
	{
		StoreHalf(&Dst[0], Src[0]);
		StoreHalf(&Dst[1], Src[1]);
		StoreHalf(&Dst[2], Src[2]);
		StoreHalf(&Dst[3], Src[3]);
	}

	static constexpr inline void WideVectorLoadHalf(float* RESTRICT Dst, const uint16* RESTRICT Src)
	{
		VectorLoadHalf(Dst, Src);
		VectorLoadHalf(Dst + 4, Src + 4);
	}

	static constexpr inline void WideVectorStoreHalf(uint16* RESTRICT Dst, const float* RESTRICT Src)
	{
		VectorStoreHalf(Dst, Src);
		VectorStoreHalf(Dst + 4, Src + 4);
	}

	/**
	 * Performs a bit cast of the given float to an unsigned int of the same bit width.
	 * @param F The float to bit cast to an unsigned integer.
	*  @return A bitwise copy of the float in a 32-bit unsigned integer value.
	 */
	[[nodiscard]] static inline uint32 AsUInt(float F)
	{
		uint32 U{};
		static_assert(sizeof(F) == sizeof(U), "The float and uint sizes must be equal");
		::memcpy(&U, &F, sizeof(F));
		return U;
	}

	/** 
	 * Performs a bit cast of the given double to an unsigned int of the same bit width.
	 * @param D The double to bit cast to an unsigned integer.
	*  @return A bitwise copy of the double in a 64-bit unsigned integer value.
	 */
	[[nodiscard]] static inline uint64 AsUInt(double D)
	{
		uint64 U{};
		static_assert(sizeof(D) == sizeof(U), "The float and uint sizes must be equal");
		::memcpy(&U, &D, sizeof(D));
		return U;
	}

	/** 
	 * Performs a bit cast of the given unsigned int to float of the same bit width.
	 * @param U The 32-bit unsigned int to bit cast to a 32-bit float.
	*  @return A bitwise copy of the 32-bit float in a 32-bit unsigned integer value.
	 */
	[[nodiscard]] static inline float AsFloat(uint32 U)
	{
		float F{};
		static_assert(sizeof(F) == sizeof(U), "The float and uint32 sizes must be equal");
		::memcpy(&F, &U, sizeof(U));
		return F;
	}

	/** 
	 * Performs a bit cast of the given unsigned int to float of the same bit width.
	 * @param U The 64-bit unsigned int to bit cast to a 64-bit float.
	*  @return A bitwise copy of the 64-bit float in a 64-bit unsigned integer value.
	 */
	[[nodiscard]] static inline double AsFloat(uint64 U)
	{
		double F{};
		static_assert(sizeof(F) == sizeof(U), "The double and uint64 sizes must be equal");
		::memcpy(&F, &U, sizeof(F));
		return F;
	}


	/**
	 * Converts a float to an integer with truncation towards zero.
	 * @param F		Floating point value to convert
	 * @return		Truncated integer.
	 */
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT int32 TruncToInt32(float F)
	{
		return (int32)F;
	}
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT int32 TruncToInt32(double F)
	{
		return (int32)F;
	}
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT int64 TruncToInt64(double F)
	{
		return (int64)F;
	}

	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT int32 TruncToInt(float F) { return TruncToInt32(F); }
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT int64 TruncToInt(double F) { return TruncToInt64(F); }

	/**
	 * Converts a float to an integer value with truncation towards zero.
	 * @param F		Floating point value to convert
	 * @return		Truncated integer value.
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT float TruncToFloat(float F)
	{
		return truncf(F);
	}

	/**
	 * Converts a double to an integer value with truncation towards zero.
	 * @param F		Floating point value to convert
	 * @return		Truncated integer value.
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT double TruncToDouble(double F)
	{
		return trunc(F);
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT double TruncToFloat(double F)
	{
		return TruncToDouble(F);
	}

	/**
	 * Converts a float to a nearest less or equal integer.
	 * @param F		Floating point value to convert
	 * @return		An integer less or equal to 'F'.
	 */
	[[nodiscard]] static inline int32 FloorToInt32(float F)
	{
		int32 I = TruncToInt32(F);
		I -= ((float)I > F);
		return I;
	}
	[[nodiscard]] static inline int32 FloorToInt32(double F)
	{
		int32 I = TruncToInt32(F);
		I -= ((double)I > F);
		return I;
	}
	[[nodiscard]] static inline int64 FloorToInt64(double F)
	{
		int64 I = TruncToInt64(F);
		I -= ((double)I > F);
		return I;
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT int32 FloorToInt(float F) { return FloorToInt32(F); }
	[[nodiscard]] static UE_FORCEINLINE_HINT int64 FloorToInt(double F) { return FloorToInt64(F); }
	
	
	/**
	* Converts a float to the nearest less or equal integer.
	* @param F		Floating point value to convert
	* @return		An integer less or equal to 'F'.
	*/
	[[nodiscard]] static UE_FORCEINLINE_HINT float FloorToFloat(float F)
	{
		return floorf(F);
	}

	/**
	* Converts a double to a less or equal integer.
	* @param F		Floating point value to convert
	* @return		The nearest integer value to 'F'.
	*/
	[[nodiscard]] static UE_FORCEINLINE_HINT double FloorToDouble(double F)
	{
		return floor(F);
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT double FloorToFloat(double F)
	{
		return FloorToDouble(F);
	}

	/**
	 * Converts a float to the nearest integer. Rounds up when the fraction is .5
	 * @param F		Floating point value to convert
	 * @return		The nearest integer to 'F'.
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT int32 RoundToInt32(float F)
	{
		return FloorToInt32(F + 0.5f);
	}
	[[nodiscard]] static UE_FORCEINLINE_HINT int32 RoundToInt32(double F)
	{
		return FloorToInt32(F + 0.5);
	}
	[[nodiscard]] static UE_FORCEINLINE_HINT int64 RoundToInt64(double F)
	{
		return FloorToInt64(F + 0.5);
	}
	
	[[nodiscard]] static UE_FORCEINLINE_HINT int32 RoundToInt(float F) { return RoundToInt32(F); }
	[[nodiscard]] static UE_FORCEINLINE_HINT int64 RoundToInt(double F) { return RoundToInt64(F); }

	/**
	* Converts a float to the nearest integer. Rounds up when the fraction is .5
	* @param F		Floating point value to convert
	* @return		The nearest integer to 'F'.
	*/
	[[nodiscard]] static UE_FORCEINLINE_HINT float RoundToFloat(float F)
	{
		return FloorToFloat(F + 0.5f);
	}

	/**
	* Converts a double to the nearest integer. Rounds up when the fraction is .5
	* @param F		Floating point value to convert
	* @return		The nearest integer to 'F'.
	*/
	[[nodiscard]] static UE_FORCEINLINE_HINT double RoundToDouble(double F)
	{
		return FloorToDouble(F + 0.5);
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT double RoundToFloat(double F)
	{
		return RoundToDouble(F);
	}

	/**
	* Converts a float to the nearest greater or equal integer.
	* @param F		Floating point value to convert
	* @return		An integer greater or equal to 'F'.
	*/
	[[nodiscard]] static inline int32 CeilToInt32(float F)
	{
		int32 I = TruncToInt32(F);
		I += ((float)I < F);
		return I;
	}
	[[nodiscard]] static inline int32 CeilToInt32(double F)
	{
		int32 I = TruncToInt32(F);
		I += ((double)I < F);
		return I;
	}
	[[nodiscard]] static inline int64 CeilToInt64(double F)
	{
		int64 I = TruncToInt64(F);
		I += ((double)I < F);
		return I;
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT int32 CeilToInt(float F) { return CeilToInt32(F); }
	[[nodiscard]] static UE_FORCEINLINE_HINT int64 CeilToInt(double F) { return CeilToInt64(F); }

	/**
	* Converts a float to the nearest greater or equal integer.
	* @param F		Floating point value to convert
	* @return		An integer greater or equal to 'F'.
	*/
	[[nodiscard]] static UE_FORCEINLINE_HINT float CeilToFloat(float F)
	{
		return ceilf(F);
	}

	/**
	* Converts a double to the nearest greater or equal integer.
	* @param F		Floating point value to convert
	* @return		An integer greater or equal to 'F'.
	*/
	[[nodiscard]] static UE_FORCEINLINE_HINT double CeilToDouble(double F)
	{
		return ceil(F);
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT double CeilToFloat(double F)
	{
		return CeilToDouble(F);
	}

	/**
	 * Converts a double to nearest int64 with ties rounding to nearest even
	 * May incur a performance penalty. Asserts on platforms that do not support this mode.
	 * @param F		Double precision floating point value to convert
	 * @return		The 64-bit integer closest to 'F', with ties going to the nearest even number
	 */
	[[nodiscard]] static int64 RoundToNearestTiesToEven(double F)
	{
#if PLATFORM_HAS_FENV_H == 0
		ensureAlwaysMsgf(FLT_ROUNDS == 1, TEXT("Platform does not support FE_TONEAREST for double to int64."));
		int64 result = llrint(F);
#else
		int PreviousRoundingMode = fegetround();
		if (PreviousRoundingMode != FE_TONEAREST)
		{
			fesetround(FE_TONEAREST);
		}
		int64 result = llrint(F);
		if (PreviousRoundingMode != FE_TONEAREST)
		{
			fesetround(PreviousRoundingMode);
		}
#endif
		return result;
	}

	/**
	* Returns signed fractional part of a float.
	* @param Value	Floating point value to convert
	* @return		A float between >=0 and < 1 for nonnegative input. A float between >= -1 and < 0 for negative input.
	*/
	[[nodiscard]] static UE_FORCEINLINE_HINT float Fractional(float Value)
	{
		return Value - TruncToFloat(Value);
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT double Fractional(double Value)
	{
		return Value - TruncToDouble(Value);
	}

	/**
	* Returns the fractional part of a float.
	* @param Value	Floating point value to convert
	* @return		A float between >=0 and < 1.
	*/
	[[nodiscard]] static UE_FORCEINLINE_HINT float Frac(float Value)
	{
		return Value - FloorToFloat(Value);
	}
	

	[[nodiscard]] static UE_FORCEINLINE_HINT double Frac(double Value)
	{
		return Value - FloorToDouble(Value);
	}

	/**
	* Breaks the given value into an integral and a fractional part.
	* @param InValue	Floating point value to convert
	* @param OutIntPart Floating point value that receives the integral part of the number.
	* @return			The fractional part of the number.
	*/
	[[nodiscard]] static UE_FORCEINLINE_HINT float Modf(const float InValue, float* OutIntPart)
	{
		return modff(InValue, OutIntPart);
	}

	/**
	* Breaks the given value into an integral and a fractional part.
	* @param InValue	Floating point value to convert
	* @param OutIntPart Floating point value that receives the integral part of the number.
	* @return			The fractional part of the number.
	*/
	[[nodiscard]] static UE_FORCEINLINE_HINT double Modf(const double InValue, double* OutIntPart)
	{
		return modf(InValue, OutIntPart);
	}

	// Returns e^Value
#if PLATFORM_WINDOWS && PLATFORM_CPU_ARM_FAMILY
	[[nodiscard]] static UE_FORCEINLINE_HINT float Exp( volatile float Value ) { return expf(Value); } // #jira UE-261901 : workaround optimization bug for expf()
#else
	[[nodiscard]] static UE_FORCEINLINE_HINT float Exp( float Value ) { return expf(Value); }
#endif
	[[nodiscard]] static UE_FORCEINLINE_HINT double Exp(double Value) { return exp(Value); }

	// Returns 2^Value
	[[nodiscard]] static UE_FORCEINLINE_HINT float Exp2( float Value ) { return powf(2.f, Value); /*exp2f(Value);*/ }
	[[nodiscard]] static UE_FORCEINLINE_HINT double Exp2(double Value) { return pow(2.0, Value); /*exp2(Value);*/ }

	[[nodiscard]] static UE_FORCEINLINE_HINT float Loge( float Value ) {	return logf(Value); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double Loge(double Value) { return log(Value); }

	[[nodiscard]] static UE_FORCEINLINE_HINT float LogX( float Base, float Value ) { return Loge(Value) / Loge(Base); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double LogX(double Base, double Value) { return Loge(Value) / Loge(Base); }
	RESOLVE_FLOAT_AMBIGUITY_2_ARGS(LogX);

	// 1.0 / Loge(2) = 1.4426950f
	[[nodiscard]] static UE_FORCEINLINE_HINT float Log2( float Value ) { return Loge(Value) * 1.4426950f; }	
	// 1.0 / Loge(2) = 1.442695040888963387
	[[nodiscard]] static UE_FORCEINLINE_HINT double Log2(double Value) { return Loge(Value) * 1.442695040888963387; }

	/**
	 * Returns the floating-point remainder of X / Y
	 * Warning: Always returns remainder toward 0, not toward the smaller multiple of Y.
	 *			So for example Fmod(2.8f, 2) gives .8f as you would expect, however, Fmod(-2.8f, 2) gives -.8f, NOT 1.2f
	 * Use Floor instead when snapping positions that can be negative to a grid
	 *
	 * This is forced to *NOT* inline so that divisions by constant Y does not get optimized in to an inverse scalar multiply,
	 * which is not consistent with the intent nor with the vectorized version.
	 */

	[[nodiscard]] static CORE_API FORCENOINLINE float Fmod(float X, float Y);
	[[nodiscard]] static CORE_API FORCENOINLINE double Fmod(double X, double Y);
	RESOLVE_FLOAT_AMBIGUITY_2_ARGS(Fmod);

	[[nodiscard]] static UE_FORCEINLINE_HINT float Sin( float Value ) { return sinf(Value); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double Sin( double Value ) { return sin(Value); }

	[[nodiscard]] static UE_FORCEINLINE_HINT float Asin( float Value ) { return asinf( (Value<-1.f) ? -1.f : ((Value<1.f) ? Value : 1.f) ); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double Asin( double Value ) { return asin( (Value<-1.0) ? -1.0 : ((Value<1.0) ? Value : 1.0) ); }

	[[nodiscard]] static UE_FORCEINLINE_HINT float Sinh(float Value) { return sinhf(Value); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double Sinh(double Value) { return sinh(Value); }

	[[nodiscard]] static UE_FORCEINLINE_HINT float Cos( float Value ) { return cosf(Value); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double Cos( double Value ) { return cos(Value); }

	[[nodiscard]] static UE_FORCEINLINE_HINT float Acos( float Value ) { return acosf( (Value<-1.f) ? -1.f : ((Value<1.f) ? Value : 1.f) ); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double Acos( double Value ) { return acos( (Value<-1.0) ? -1.0 : ((Value<1.0) ? Value : 1.0) ); }

	[[nodiscard]] static UE_FORCEINLINE_HINT float Cosh(float Value) { return coshf(Value); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double Cosh(double Value) { return cosh(Value); }

	[[nodiscard]] static UE_FORCEINLINE_HINT float Tan( float Value ) { return tanf(Value); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double Tan( double Value ) { return tan(Value); }

	[[nodiscard]] static UE_FORCEINLINE_HINT float Atan( float Value ) { return atanf(Value); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double Atan( double Value ) { return atan(Value); }

	[[nodiscard]] static UE_FORCEINLINE_HINT float Tanh(float Value) { return tanhf(Value); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double Tanh(double Value) { return tanh(Value); }

	[[nodiscard]] static CORE_API float Atan2( float Y, float X );
	[[nodiscard]] static CORE_API double Atan2( double Y, double X );
	RESOLVE_FLOAT_AMBIGUITY_2_ARGS(Atan2);

	[[nodiscard]] static UE_FORCEINLINE_HINT float Sqrt( float Value ) { return sqrtf(Value); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double Sqrt( double Value ) { return sqrt(Value); }

	[[nodiscard]] static UE_FORCEINLINE_HINT float Pow( float A, float B ) { return powf(A,B); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double Pow( double A, double B ) { return pow(A,B); }
	RESOLVE_FLOAT_AMBIGUITY_2_ARGS(Pow);

	/** Computes a fully accurate inverse square root */
	[[nodiscard]] static UE_FORCEINLINE_HINT float InvSqrt( float F ) { return 1.0f / sqrtf( F ); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double InvSqrt( double F ) { return 1.0 / sqrt( F ); }

	/** Computes a faster but less accurate inverse square root */
	[[nodiscard]] static UE_FORCEINLINE_HINT float InvSqrtEst( float F ) { return InvSqrt( F ); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double InvSqrtEst( double F ) { return InvSqrt( F ); }

	/** Composes a floating-point value with the magnitude of X and the sign of Y. */
	[[nodiscard]] static UE_FORCEINLINE_HINT float CopySign(float X, float Y) { return copysignf(X, Y); }
	[[nodiscard]] static UE_FORCEINLINE_HINT double CopySign(double X, double Y) { return copysign(X, Y); }
	RESOLVE_FLOAT_AMBIGUITY_2_ARGS(CopySign);

	/** Return true if value is NaN (not a number). */
	[[nodiscard]] static UE_FORCEINLINE_HINT bool IsNaN( float A ) 
	{
		return (BitCast<uint32>(A) & 0x7FFFFFFFU) > 0x7F800000U;
	}
	[[nodiscard]] static UE_FORCEINLINE_HINT bool IsNaN(double A)
	{
		return (BitCast<uint64>(A) & 0x7FFFFFFFFFFFFFFFULL) > 0x7FF0000000000000ULL;
	}

	/** Return true if value is finite (not NaN and not Infinity). */
	[[nodiscard]] static UE_FORCEINLINE_HINT bool IsFinite( float A )
	{
		return (BitCast<uint32>(A) & 0x7F800000U) != 0x7F800000U;
	}
	[[nodiscard]] static UE_FORCEINLINE_HINT bool IsFinite(double A)
	{
		return (BitCast<uint64>(A) & 0x7FF0000000000000ULL) != 0x7FF0000000000000ULL;
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT bool IsNegativeOrNegativeZero(float A)
	{
		return BitCast<uint32>(A) >= (uint32)0x80000000; // Detects sign bit.
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT bool IsNegativeOrNegativeZero(double A)
	{
		return BitCast<uint64>(A) >= (uint64)0x8000000000000000; // Detects sign bit.
	}

	/** Returns a random integer between 0 and RAND_MAX, inclusive */
	[[nodiscard]] static UE_FORCEINLINE_HINT int32 Rand() { return rand(); }

	/** Returns a random integer between 0 and MAX_int32, inclusive. RAND_MAX may only be 15 bits, so compose from multiple calls. */
	[[nodiscard]] static UE_FORCEINLINE_HINT int32 Rand32() { return ((rand() & 0x7fff) << 16) | ((rand() & 0x7fff) << 1) | (rand() & 0x1); }

	/** Seeds global random number functions Rand() and FRand() */
	static UE_FORCEINLINE_HINT void RandInit(int32 Seed) { srand( Seed ); }

	/** Returns a random float between 0 and 1, inclusive. */
	[[nodiscard]] static inline float FRand() 
	{ 
		// FP32 mantissa can only represent 24 bits before losing precision
		constexpr int32 RandMax = 0x00ffffff < RAND_MAX ? 0x00ffffff : RAND_MAX;
		return (Rand() & RandMax) / (float)RandMax;
	}

	/** Seeds future calls to SRand() */
	static CORE_API void SRandInit( int32 Seed );

	/** Returns the current seed for SRand(). */
	[[nodiscard]] static CORE_API int32 GetRandSeed();

	/** Returns a seeded random float in the range [0,1), using the seed from SRandInit(). */
	[[nodiscard]] static CORE_API float SRand();

	/**
	 * Computes the base 2 logarithm for an integer value.
	 * The result is rounded down to the nearest integer.
	 *
	 * @param Value		The value to compute the log of
	 * @return			Log2 of Value. 0 if Value is 0.
	 */	
	[[nodiscard]] static constexpr inline uint32 FloorLog2(uint32 Value) 
	{
		uint32 pos = 0;
		if (Value >= 1<<16) { Value >>= 16; pos += 16; }
		if (Value >= 1<< 8) { Value >>=  8; pos +=  8; }
		if (Value >= 1<< 4) { Value >>=  4; pos +=  4; }
		if (Value >= 1<< 2) { Value >>=  2; pos +=  2; }
		if (Value >= 1<< 1) {				pos +=  1; }
		return pos;
	}

	/** FloorLog2 but the caller guarantees that Value is not 0. */
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT uint32 FloorLog2NonZero(uint32 Value)
	{
		return FloorLog2(Value);
	}

	/**
	 * Computes the base 2 logarithm for a 64-bit value.
	 * The result is rounded down to the nearest integer.
	 *
	 * @param Value		The value to compute the log of
	 * @return			Log2 of Value. 0 if Value is 0.
	 */	
	[[nodiscard]] static constexpr inline uint64 FloorLog2_64(uint64 Value) 
	{
		uint64 pos = 0;
		if (Value >= 1ull<<32) { Value >>= 32; pos += 32; }
		if (Value >= 1ull<<16) { Value >>= 16; pos += 16; }
		if (Value >= 1ull<< 8) { Value >>=  8; pos +=  8; }
		if (Value >= 1ull<< 4) { Value >>=  4; pos +=  4; }
		if (Value >= 1ull<< 2) { Value >>=  2; pos +=  2; }
		if (Value >= 1ull<< 1) {               pos +=  1; }
		return pos;
	}

	/** FloorLog2_64 but the caller guarantees that Value is not 0. */
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT uint64 FloorLog2NonZero_64(uint64 Value)
	{
		return FloorLog2_64(Value);
	}

	/**
	 * Counts the number of leading zeros in the bit representation of the 8-bit value
	 *
	 * @param Value the value to determine the number of leading zeros for
	 *
	 * @return the number of zeros before the first "on" bit
	 */
	[[nodiscard]] static constexpr inline uint8 CountLeadingZeros8(uint8 Value)
	{
		if (Value == 0) return 8;
		return uint8(7 - FloorLog2(uint32(Value)));
	}

	/**
	 * Counts the number of leading zeros in the bit representation of the 32-bit value
	 *
	 * @param Value the value to determine the number of leading zeros for
	 *
	 * @return the number of zeros before the first "on" bit
	 */
	[[nodiscard]] static constexpr inline uint32 CountLeadingZeros(uint32 Value)
	{
		if (Value == 0) return 32;
		return 31 - FloorLog2(Value);
	}

	/**
	 * Counts the number of leading zeros in the bit representation of the 64-bit value
	 *
	 * @param Value the value to determine the number of leading zeros for
	 *
	 * @return the number of zeros before the first "on" bit
	 */
	[[nodiscard]] static constexpr inline uint64 CountLeadingZeros64(uint64 Value)
	{
		if (Value == 0) return 64;
		return 63 - FloorLog2_64(Value);
	}

	/**
	 * Counts the number of trailing zeros in the bit representation of the value
	 *
	 * @param Value the value to determine the number of trailing zeros for
	 *
	 * @return the number of zeros after the last "on" bit
	 */
	[[nodiscard]] static constexpr inline uint32 CountTrailingZeros(uint32 Value)
	{
		if (Value == 0)
		{
			return 32;
		}
		uint32 Result = 0;
		while ((Value & 1) == 0)
		{
			Value >>= 1;
			++Result;
		}
		return Result;
	}

	UE_DEPRECATED(5.7, "Please use CountTrailingZeros instead")
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT uint32 CountTrailingZerosConstExpr(uint32 Value)
	{
		return CountTrailingZeros(Value);
	}

	/**
	 * Counts the number of trailing zeros in the bit representation of the value
	 *
	 * @param Value the value to determine the number of trailing zeros for
	 *
	 * @return the number of zeros after the last "on" bit
	 */
	[[nodiscard]] static constexpr inline uint64 CountTrailingZeros64(uint64 Value)
	{
		if (Value == 0)
		{
			return 64;
		}
		uint64 Result = 0;
		while ((Value & 1) == 0)
		{
			Value >>= 1;
			++Result;
		}
		return Result;
	}

	UE_DEPRECATED(5.7, "Please use CountTrailingZeros64 instead")
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT uint64 CountTrailingZeros64ConstExpr(uint64 Value)
	{
		return CountTrailingZeros64(Value);
	}

	/**
	 * Returns smallest N such that (1<<N)>=Arg.
	 * Note: CeilLogTwo(0)=0 
	 */
	[[nodiscard]] static constexpr inline uint32 CeilLogTwo( uint32 Arg )
	{
		// if Arg is 0, change it to 1 so that we return 0
		Arg = Arg ? Arg : 1;
		return 32 - CountLeadingZeros(Arg - 1);
	}

	[[nodiscard]] static constexpr inline uint64 CeilLogTwo64( uint64 Arg )
	{
		// if Arg is 0, change it to 1 so that we return 0
		Arg = Arg ? Arg : 1;
		return 64 - CountLeadingZeros64(Arg - 1);
	}

	/**
	 * Returns the smallest N such that (1<<N)>=Arg. This is a less efficient version of CeilLogTwo, but written in a
	 * way that can be evaluated at compile-time.
	 */
	[[nodiscard]] static constexpr inline uint8 ConstExprCeilLogTwo(SIZE_T Arg)
	{
		if (Arg <= 1)
		{
			return 0;
		}
		// Integer overflow if we tried to add 1 to maximum value, so handle that case separately
		if (Arg + 1 < Arg)
		{
			return sizeof(Arg) * 8;
		}
		return 1 + ConstExprCeilLogTwo((Arg + 1) / 2);
	}

	/** @return Rounds the given number up to the next highest power of two. */
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT uint32 RoundUpToPowerOfTwo(uint32 Arg)
	{
		return 1u << CeilLogTwo(Arg);
	}

	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT uint64 RoundUpToPowerOfTwo64(uint64 V)
	{
		return uint64(1) << CeilLogTwo64(V);
	}

	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT uint64 RoundDownToPowerOfTwo64(uint64 V)
	{
		return (V == 0) ? 0u : (uint64(1) << FloorLog2_64(V));
	}

	/** Spreads bits to every other. */
	[[nodiscard]] static constexpr inline uint32 MortonCode2( uint32 x )
	{
		x &= 0x0000ffff;
		x = (x ^ (x << 8)) & 0x00ff00ff;
		x = (x ^ (x << 4)) & 0x0f0f0f0f;
		x = (x ^ (x << 2)) & 0x33333333;
		x = (x ^ (x << 1)) & 0x55555555;
		return x;
	}

	[[nodiscard]] static constexpr inline uint64 MortonCode2_64( uint64 x )
	{
		x &= 0x00000000ffffffff;
		x = (x ^ (x << 16)) & 0x0000ffff0000ffff;
		x = (x ^ (x << 8)) & 0x00ff00ff00ff00ff;
		x = (x ^ (x << 4)) & 0x0f0f0f0f0f0f0f0f;
		x = (x ^ (x << 2)) & 0x3333333333333333;
		x = (x ^ (x << 1)) & 0x5555555555555555;
		return x;
	}

	/** Reverses MortonCode2. Compacts every other bit to the right. */
	[[nodiscard]] static constexpr inline uint32 ReverseMortonCode2( uint32 x )
	{
		x &= 0x55555555;
		x = (x ^ (x >> 1)) & 0x33333333;
		x = (x ^ (x >> 2)) & 0x0f0f0f0f;
		x = (x ^ (x >> 4)) & 0x00ff00ff;
		x = (x ^ (x >> 8)) & 0x0000ffff;
		return x;
	}

	[[nodiscard]] static constexpr inline uint64 ReverseMortonCode2_64( uint64 x )
	{
		x &= 0x5555555555555555;
		x = (x ^ (x >> 1)) & 0x3333333333333333;
		x = (x ^ (x >> 2)) & 0x0f0f0f0f0f0f0f0f;
		x = (x ^ (x >> 4)) & 0x00ff00ff00ff00ff;
		x = (x ^ (x >> 8)) & 0x0000ffff0000ffff;
		x = (x ^ (x >> 16)) & 0x00000000ffffffff;
		return x;
	}

	/** Spreads bits to every 3rd. */
	[[nodiscard]] static constexpr inline uint32 MortonCode3( uint32 x )
	{
		x &= 0x000003ff;
		x = (x ^ (x << 16)) & 0xff0000ff;
		x = (x ^ (x <<  8)) & 0x0300f00f;
		x = (x ^ (x <<  4)) & 0x030c30c3;
		x = (x ^ (x <<  2)) & 0x09249249;
		return x;
	}

	/** Reverses MortonCode3. Compacts every 3rd bit to the right. */
	[[nodiscard]] static constexpr inline uint32 ReverseMortonCode3( uint32 x )
	{
		x &= 0x09249249;
		x = (x ^ (x >>  2)) & 0x030c30c3;
		x = (x ^ (x >>  4)) & 0x0300f00f;
		x = (x ^ (x >>  8)) & 0xff0000ff;
		x = (x ^ (x >> 16)) & 0x000003ff;
		return x;
	}

	/**
	 * Returns value based on comparand. The main purpose of this function is to avoid
	 * branching based on floating point comparison which can be avoided via compiler
	 * intrinsics.
	 *
	 * Please note that we don't define what happens in the case of NaNs as there might
	 * be platform specific differences.
	 *
	 * @param	Comparand		Comparand the results are based on
	 * @param	ValueGEZero		Return value if Comparand >= 0
	 * @param	ValueLTZero		Return value if Comparand < 0
	 *
	 * @return	ValueGEZero if Comparand >= 0, ValueLTZero otherwise
	 */
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT float FloatSelect( float Comparand, float ValueGEZero, float ValueLTZero )
	{
		return Comparand >= 0.f ? ValueGEZero : ValueLTZero;
	}

	/**
	 * Returns value based on comparand. The main purpose of this function is to avoid
	 * branching based on floating point comparison which can be avoided via compiler
	 * intrinsics.
	 *
	 * Please note that we don't define what happens in the case of NaNs as there might
	 * be platform specific differences.
	 *
	 * @param	Comparand		Comparand the results are based on
	 * @param	ValueGEZero		Return value if Comparand >= 0
	 * @param	ValueLTZero		Return value if Comparand < 0
	 *
	 * @return	ValueGEZero if Comparand >= 0, ValueLTZero otherwise
	 */
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT double FloatSelect( double Comparand, double ValueGEZero, double ValueLTZero )
	{
		return Comparand >= 0.0 ? ValueGEZero : ValueLTZero;
	}

	/** Computes absolute value in a generic way */
	template< class T > 
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT T Abs( const T A )
	{
		return (A < (T)0) ? (T)-A : A;
	}

	/** Returns 1, 0, or -1 depending on relation of T to 0 */
	template< class T > 
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT T Sign( const T A )
	{
		return (A > (T)0) ? (T)1 : ((A < (T)0) ? (T)-1 : (T)0);
	}

	/** Returns higher value in a generic way */
	template <typename T> 
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT T Max(T A, T B)
	{
		// Even though this should be covered by the variadic, we still need this overload because of the many instances of
		// FMath::Max<T>(a, b) with an explicit template parameter, which needs to continue to be supported.

		return (B < A) ? A : B;
	}
	template <
		typename T, typename... OtherTypes
		UE_REQUIRES((std::is_same_v<T, OtherTypes> && ...))
	>
	[[nodiscard]] static constexpr inline T Max(T A, OtherTypes... Others)
	{
		if constexpr (sizeof...(OtherTypes) > 0)
		{
			A = FGenericPlatformMath::Max(A, FGenericPlatformMath::Max(Others...));
		}
		return A;
	}

	/** Returns lower value in a generic way */
	template <typename T> 
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT T Min(T A, T B)
	{
		// Even though this should be covered by the variadic, we still need this overload because of the many instances of
		// FMath::Min<T>(a, b) with an explicit template parameter, which needs to continue to be supported.

		return (A < B) ? A : B;
	}
	template <
		typename T, typename... OtherTypes
		UE_REQUIRES((std::is_same_v<T, OtherTypes> && ...))
	>
	[[nodiscard]] static constexpr inline T Min(T A, OtherTypes... Others)
	{
		if constexpr (sizeof...(OtherTypes) > 0)
		{
			A = FGenericPlatformMath::Min(A, FGenericPlatformMath::Min(Others...));
		}
		return A;
	}

	// Allow mixing of float types to promote to highest precision type
	MIX_FLOATS_2_ARGS(Max);
	MIX_FLOATS_2_ARGS(Min);

	// Allow mixing of signed integral types.
	MIX_SIGNED_INTS_2_ARGS_CONSTEXPR(Max);
	MIX_SIGNED_INTS_2_ARGS_CONSTEXPR(Min);

	/**
	* Min of Array
	* @param	Values Array of templated type
	* @return	The min value found in the array or default value if the array was empty
	*/
	template< class T >
	[[nodiscard]] static inline T Min(const TArray<T>& Values)
	{
		if (Values.Num() == 0)
		{
			return T();
		}

		T CurMin = Values[0];
		for (int32 v = 1; v < Values.Num(); ++v)
		{
			const T Value = Values[v];
			if (Value < CurMin)
			{
				CurMin = Value;
			}
		}
		return CurMin;
	}
	
	/**
	* Min of Array
	* @param	Values Array of templated type
	* @param	MinIndex Optional pointer for returning the index of the minimum element, if multiple minimum elements the first index is returned
	* @return	The min value found in the array or default value if the array was empty
	*/
	template< class T >
	static inline T Min(const TArray<T>& Values, int32* MinIndex)
	{
		if (Values.Num() == 0)
		{
			if (MinIndex)
			{
				*MinIndex = INDEX_NONE;
			}
			return T();
		}

		T CurMin = Values[0];
		int32 CurMinIndex = 0;
		for (int32 v = 1; v < Values.Num(); ++v)
		{
			const T Value = Values[v];
			if (Value < CurMin)
			{
				CurMin = Value;
				CurMinIndex = v;
			}
		}

		if (MinIndex)
		{
			*MinIndex = CurMinIndex;
		}
		return CurMin;
	}	
	
	/**
* Max of Array
* @param	Values Array of templated type
* @return	The max value found in the array or default value if the array was empty
*/
	template< class T >
	[[nodiscard]] static inline T Max(const TArray<T>& Values)
	{
		if (Values.Num() == 0)
		{
			return T();
		}

		T CurMax = Values[0];
		for (int32 v = 1; v < Values.Num(); ++v)
		{
			const T Value = Values[v];
			if (CurMax < Value)
			{
				CurMax = Value;
			}
		}

		return CurMax;
	}

	/**
	* Max of Array
	* @param	Values Array of templated type
	* @param	MaxIndex Optional pointer for returning the index of the maximum element, if multiple maximum elements the first index is returned
	* @return	The max value found in the array or default value if the array was empty
	*/
	template< class T >
	static inline T Max(const TArray<T>& Values, int32* MaxIndex)
	{
		if (Values.Num() == 0)
		{
			if (MaxIndex)
			{
				*MaxIndex = INDEX_NONE;
			}
			return T();
		}

		T CurMax = Values[0];
		int32 CurMaxIndex = 0;
		for (int32 v = 1; v < Values.Num(); ++v)
		{
			const T Value = Values[v];
			if (CurMax < Value)
			{
				CurMax = Value;
				CurMaxIndex = v;
			}
		}

		if (MaxIndex)
		{
			*MaxIndex = CurMaxIndex;
		}
		return CurMax;
	}

	/**
	 * Sort input A & B values into OutMin & OutMax
	 */
	template< class T >
	static constexpr inline void GetMinMax(const T& A, const T& B, T& OutMin, T& OutMax)
	{
		if (A < B)
		{
			OutMin = A;
			OutMax = B;
		}
		else
		{
			OutMin = B;
			OutMax = A;
		}
	}

	/**
	 * Sort input values to be Min and Max
	 */
	template< class T >
	static constexpr inline void GetMinMax(T& Min, T& Max)
	{
		if (Max < Min)
		{
			Swap(Min, Max);
		}
	}

	[[nodiscard]] static constexpr inline int32 CountBits(uint32 Bits)
	{
		// https://en.wikipedia.org/wiki/Hamming_weight
		Bits -= (Bits >> 1) & 0x55555555u;
		Bits = (Bits & 0x33333333u) + ((Bits >> 2) & 0x33333333u);
		Bits = (Bits + (Bits >> 4)) & 0x0f0f0f0fu;
		return (Bits * 0x01010101u) >> 24;
	}

	[[nodiscard]] static constexpr inline int32 CountBits64(uint64 Bits)
	{
		// https://en.wikipedia.org/wiki/Hamming_weight
		Bits -= (Bits >> 1) & 0x5555555555555555ull;
		Bits = (Bits & 0x3333333333333333ull) + ((Bits >> 2) & 0x3333333333333333ull);
		Bits = (Bits + (Bits >> 4)) & 0x0f0f0f0f0f0f0f0full;
		return (Bits * 0x0101010101010101) >> 56;
	}

	template <UE::CIntegral T>
		requires (sizeof(T) > 4)
	[[nodiscard]] UE_REWRITE static constexpr int32 CountBits(T Bits)
	{
		UE_STATIC_DEPRECATE(5.8, sizeof(T) > 4, "CountBits for 64-bit values has been deprecated, please use CountBits64 instead.");
		return CountBits64(Bits);
	}

#if WITH_DEV_AUTOMATION_TESTS
	/** Test some of the tricky functions above **/
	static void AutoTest();
#endif

	/**
	 * Adds two integers of any integer type, checking for overflow.
	 * If there was overflow, it returns false, and OutResult may or may not be written.
	 * If there wasn't overflow, it returns true, and the result of the addition is written to OutResult.
	 */
	template <UE::CIntegral IntType>
	[[nodiscard]] static inline bool AddAndCheckForOverflow(IntType A, IntType B, IntType& OutResult)
	{
		// This follows Hacker's Delight, Chapter 2-12
		// Signed->unsigned conversion and unsigned addition have defined behavior always
		typedef typename std::make_unsigned_t<IntType> UnsignedType;
		const UnsignedType UnsignedA = static_cast<UnsignedType>(A);
		const UnsignedType UnsignedB = static_cast<UnsignedType>(B);
		const UnsignedType UnsignedSum = UnsignedA + UnsignedB;

		if constexpr (std::is_signed_v<IntType>)
		{
			// Check for signed overflow.
			// The underlying logic here is pretty simple: if A and B had opposite signs, their sum can't
			// overflow. If they had the same sign and the sum has the opposite value in the sign bit, we
			// had an overflow. (See Hacker's Delight Chapter 2-12 for more details.)
			constexpr UnsignedType UnsignedMSB = static_cast<UnsignedType>(std::numeric_limits<IntType>::min());
			if((UnsignedSum ^ UnsignedA) & (UnsignedSum ^ UnsignedB) & UnsignedMSB)
			{
				return false;
			}
			else
			{
				OutResult = A + B;
				return true;
			}
		}
		else
		{
			// Iff there was overflow, the modular sum must be less than both the operands.
			if (UnsignedSum >= UnsignedA)
			{
				OutResult = UnsignedSum;
				return true;
			}
			else
			{
				return false;
			}
		}
	}

	/**
	 * Subtracts two integers of any integer type, checking for overflow.
	 * If there was overflow, it returns false, and OutResult may or may not be written.
	 * If there wasn't overflow, it returns true, and the result of the subtraction is written to OutResult.
	 */
	template <UE::CIntegral IntType>
	[[nodiscard]] static inline bool SubtractAndCheckForOverflow(IntType A, IntType B, IntType& OutResult)
	{
		// This follows Hacker's Delight, Chapter 2-12
		// Signed->unsigned conversion and unsigned addition have defined behavior always
		typedef typename std::make_unsigned_t<IntType> UnsignedType;
		const UnsignedType UnsignedA = static_cast<UnsignedType>(A);
		const UnsignedType UnsignedB = static_cast<UnsignedType>(B);
		const UnsignedType UnsignedDiff = UnsignedA - UnsignedB;

		if constexpr (std::is_signed_v<IntType>)
		{
			// Check for signed overflow.
			// If A and B have the same sign, the difference can't overflow. Therefore, we test for cases
			// where the sign bit differs meaning ((UnsignedA ^ UnsignedB) & UnsignedMSB) != 0, and
			// simultaneously the sign of the difference differs from the sign of the minuend (which should
			// keep its sign when we're subtracting a value of the opposite sign), meaning
			// ((UnsignedDiff ^ UnsignedA) & UnsignedMSB) != 0. Combining the two yields:
			constexpr UnsignedType UnsignedMSB = static_cast<UnsignedType>(std::numeric_limits<IntType>::min());
			if ((UnsignedA ^ UnsignedB) & (UnsignedDiff ^ UnsignedA) & UnsignedMSB)
			{
				return false;
			}
			else
			{
				OutResult = A - B;
				return true;
			}
		}
		else
		{
			OutResult = UnsignedDiff;
			return A >= B;
		}
	}

	/**
	 * Multiplies two integers of any integer type, checking for overflow.
	 * If there was overflow, it returns false, and OutResult may or may not be written.
	 * If there wasn't overflow, it returns true, and the result of the multiplication is written to OutResult.
	 */
	template <UE::CIntegral IntType>
	[[nodiscard]] static inline bool MultiplyAndCheckForOverflow(IntType A, IntType B, IntType& OutResult)
	{
		// Handle the case where the second factor is 0 specially (why will become clear in a minute).
		if (B == 0)
		{
			// Anything times 0 is 0.
			OutResult = 0;
			return true;
		}

		if constexpr (std::is_signed_v<IntType>)
		{
			// The overflow check is annoying and expensive, but the basic idea is fairly simple:
			// reduce to an unsigned check of the absolute values. (Again the basic algorithm is
			// in Hacker's Delight, Chapter 2-12).
			//
			// We need the absolute value of the product to be <=MaxValue when the result is positive
			// (signs of factors same) and <= -MinValue = MaxValue + 1 if the result is negative
			// (signs of factors opposite).
			typedef typename std::make_unsigned_t<IntType> UnsignedType;
			UnsignedType UnsignedA = static_cast<UnsignedType>(A);
			UnsignedType UnsignedB = static_cast<UnsignedType>(B);
			bool bSignsDiffer = false;

			// Determine the unsigned absolute values of A and B carefully (note we can't negate signed
			// A or B, because negating MinValue is UB). We can however subtract their
			// unsigned values from 0 if the original value was less than zero. While doing this, also
			// keep track of the sign parity.
			if (A < 0)
			{
				UnsignedA = UnsignedType(0) - UnsignedA;
				bSignsDiffer = !bSignsDiffer;
			}

			if (B < 0)
			{
				UnsignedB = UnsignedType(0) - UnsignedB;
				bSignsDiffer = !bSignsDiffer;
			}

			// Determine the unsigned product bound we need based on whether the signs were same or different.
			const UnsignedType ProductBound = UnsignedType(std::numeric_limits<IntType>::max()) + (bSignsDiffer ? 1 : 0);

			// We're now in the unsigned case, 0 <= UnsignedA, 0 < UnsignedB (we established b != 0), and for
			// there not to be overflows we need
			//   a * b <= ProductBound
			// <=> a <= ProductBound/b
			// <=> a <= floor(ProductBound/b)   since a is integer
			if (UnsignedA <= ProductBound / UnsignedB)
			{
				OutResult = A * B;
				return true;
			}
			else
			{
				return false;
			}
		}
		else
		{
			OutResult = A * B;
			return OutResult / B == A;
		}
	}

private:

	/** Error reporting for Fmod. Not inlined to avoid compilation issues and avoid all the checks and error reporting at all callsites. */
	static CORE_API void FmodReportError(float X, float Y);
	static CORE_API void FmodReportError(double X, double Y);
};

/** Float specialization */
template<>
[[nodiscard]] UE_FORCEINLINE_HINT float FGenericPlatformMath::Abs( const float A )
{
	return fabsf( A );
}
template<>
[[nodiscard]] UE_FORCEINLINE_HINT double FGenericPlatformMath::Abs(const double A)
{
	return fabs(A);
}
