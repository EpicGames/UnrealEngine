// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	GenericPlatformMemory.h: Generic platform memory classes
==============================================================================================*/

#pragma once

#include "CoreFwd.h" // IWYU pragma: keep
#include "CoreTypes.h"
#include "HAL/PlatformString.h"

#include <string.h>
#include <wchar.h>

class FMalloc;
class FOutputDevice;
class FString;
/** Holds generic memory stats, internally implemented as a map. */
struct FGenericMemoryStats;
struct FPlatformMemoryStats;

#ifndef UE_CHECK_LARGE_ALLOCATIONS
#define UE_CHECK_LARGE_ALLOCATIONS 0
#endif

#if UE_CHECK_LARGE_ALLOCATIONS
class FAutoConsoleVariableRef;
namespace UE::Memory::Private
{
	extern bool GEnableLargeAllocationChecks;
	extern int32 GLargeAllocationThreshold;
	extern FAutoConsoleVariableRef CVarEnableLargeAllocationChecks;
}
#endif

/**
* Platform-dependent "bucket" for memory size, where Default is the normal, or possibly the largest.
* This is generally used for texture LOD settings for how to fit in smaller memory devices
*/
#define PLATFORM_MEMORY_SIZE_BUCKET_LIST(XBUCKET) \
	/* not used with texture LODs (you can't use bigger textures than what is cooked out, which is what Default should map to) */ \
	XBUCKET(Largest)	\
	XBUCKET(Larger)		\
	/* these are used by texture LODs */ \
	XBUCKET(Default)	\
	XBUCKET(Smaller)	\
	XBUCKET(Smallest)	\
	XBUCKET(Tiniest)	\

#define PLATFORM_MEMORY_SIZE_BUCKET_ENUM(Name) Name,
enum class EPlatformMemorySizeBucket
{
	PLATFORM_MEMORY_SIZE_BUCKET_LIST(PLATFORM_MEMORY_SIZE_BUCKET_ENUM)
};
#undef PLATFORM_MEMORY_SIZE_BUCKET_ENUM

inline const TCHAR* LexToString(EPlatformMemorySizeBucket Bucket)
{
#define PLATFORM_MEMORY_SIZE_BUCKET_LEXTOSTRING(Name) case EPlatformMemorySizeBucket::Name: return TEXT(#Name);
	switch (Bucket)
	{
		PLATFORM_MEMORY_SIZE_BUCKET_LIST(PLATFORM_MEMORY_SIZE_BUCKET_LEXTOSTRING)
	}
#undef PLATFORM_MEMORY_SIZE_BUCKET_LEXTOSTRING

	return TEXT("Unknown");
}

enum class EMemcpyCachePolicy : uint8
{
	// Writes to destination memory are cache-visible (default).
	// This should be used if copy results are immediately accessed by CPU.
	StoreCached,

	// Writes to destination memory bypass cache (avoiding pollution).
	// Optimizes for large copies that aren't read from soon after.
	StoreUncached,
};

/** 
 * Struct used to hold common memory constants for all platforms.
 * These values generally don't change over the life of the executable, except for memory hotplugging and the addition or removal of swap space.
 */
struct FGenericPlatformMemoryConstants
{
	/** The amount of actual physical memory, in bytes. */
	uint64 TotalPhysical = 0;

	/** The amount of virtual memory, in bytes. */
	uint64 TotalVirtual = 0;

	/** The size of a physical page, in bytes. This is also the granularity for PageProtect(), commitment and properties (e.g. ability to access) of the physical RAM. */
	SIZE_T PageSize = 0;

	/**
	 * Some platforms have advantages if memory is allocated in chunks larger than PageSize (e.g. VirtualAlloc() seems to have 64KB granularity as of now).
     * This value is the minimum allocation size that the system will use behind the scenes.
	 */
	SIZE_T OsAllocationGranularity = 0;

	/** The size of a "page" in Binned2 malloc terms, in bytes. Should be at least 64KB. BinnedMalloc expects memory returned from BinnedAllocFromOS() to be aligned on BinnedPageSize boundary. */
	SIZE_T BinnedPageSize = 0;

	/** This is the "allocation granularity" in Binned malloc terms, i.e. BinnedMalloc will allocate the memory in increments of this value. If zero, Binned will use BinnedPageSize for this value. */
	SIZE_T BinnedAllocationGranularity = 0;

	/**
	 * Starting address for the available virtual address space.
	 * Can be used with AddressLimit to determine address space range for binned allocators
	 */
	uint64 AddressStart = 0;

	/**
	 * An estimate of the range of addresses expected to be returned by BinnedAllocFromOS(). Binned
	 * Malloc will adjust its internal structures to make lookups for memory allocations O(1) for this range. 
	 * It is ok to go outside this range, lookups will just be a little slower
	 */
	uint64 AddressLimit = (uint64)0xffffffff + 1;

	/** Approximate physical RAM in GB; 1 on everything except PC. Used for "course tuning", like FPlatformMisc::NumberOfCores(). */
	uint32 TotalPhysicalGB = 1;
};

typedef FGenericPlatformMemoryConstants FPlatformMemoryConstants;

/** 
 * Struct used to hold common memory stats for all platforms.
 * These values may change over the entire life of the executable.
 */
struct FGenericPlatformMemoryStats : public FPlatformMemoryConstants
{
	/** The amount of physical memory currently available, in bytes. */
	uint64 AvailablePhysical;

	/** The amount of virtual memory currently available, in bytes. */
	uint64 AvailableVirtual;

	/** The amount of physical memory used by the process, in bytes. */
	uint64 UsedPhysical;

	/** The peak amount of physical memory used by the process, in bytes. */
	uint64 PeakUsedPhysical;

	/** Total amount of virtual memory used by the process. */
	uint64 UsedVirtual;

	/** The peak amount of virtual memory used by the process. */
	uint64 PeakUsedVirtual;
	
	/** Memory pressure states, useful for platforms in which the available memory estimate
	 	may not take in to account memory reclaimable from closing inactive processes or resorting to swap. */
	enum class EMemoryPressureStatus : uint8 
	{ 
		Unknown,
		Nominal,
		Warning,
		Critical, // high risk of OOM conditions
	};
	CORE_API EMemoryPressureStatus GetMemoryPressureStatus() const;

	/** Default constructor, clears all variables. */
	CORE_API FGenericPlatformMemoryStats();

	struct FPlatformSpecificStat
	{
		const TCHAR* Name;
		uint64 Value;

		FPlatformSpecificStat(const TCHAR* InName, uint64 InValue)
			: Name(InName)
			, Value(InValue)
		{}
	};

	CORE_API TArray<FPlatformSpecificStat> GetPlatformSpecificStats() const;

	CORE_API uint64 GetAvailablePhysical(bool bExcludeExtraDevMemory) const;

	/** Called by FCsvProfiler::EndFrame to set platform specific CSV stats. */
	void SetEndFrameCsvStats() const {}
};

// Contains shared/private information for a single page allocation from the kernel. A page
// allocation may contain many pages.
struct FForkedPageAllocation
{
	// Start/End virtual address for the allocation.
	uint64 PageStart;
	uint64 PageEnd;

	// The amount of memory in this allocation range that is shared across the forked
	// child processes.
	uint64 SharedCleanKiB;
	uint64 SharedDirtyKiB;

	// The amount of memory in this allocation range that has been written to by the child
	// process, and as a result has been made unique to the process.
	uint64 PrivateCleanKiB;
	uint64 PrivateDirtyKiB;
};



struct FPlatformMemoryStats;

/**
 * FMemory_Alloca/alloca implementation. This can't be a function, even FORCEINLINE'd because there's no guarantee that 
 * the memory returned in a function will stick around for the caller to use.
 */
#if PLATFORM_USES_MICROSOFT_LIBC_FUNCTIONS
#define __FMemory_Alloca_Func _alloca
#else
#define __FMemory_Alloca_Func alloca
#endif

#define FMemory_Alloca(Size) ((Size==0) ? 0 : (void*)(((PTRINT)__FMemory_Alloca_Func(Size + 15) + 15) & ~15))

 // Version that supports alignment requirement. However since the old alignment was always forced to be 16, this continues to enforce a min alignment of 16 but allows a larger value.
#define FMemory_Alloca_Aligned(Size, Alignment) ((Size==0) ? 0 : ((Alignment <= 16) ? FMemory_Alloca(Size) : (void*)(((PTRINT)__FMemory_Alloca_Func(Size + Alignment-1) + Alignment-1) & ~(Alignment-1))))

/** Generic implementation for most platforms, these tend to be unused and unimplemented. */
struct FGenericPlatformMemory
{
	/** Set to true if we encounters out of memory. */
	static CORE_API bool bIsOOM;

	/** Set to size of allocation that triggered out of memory, zero otherwise. */
	static CORE_API uint64 OOMAllocationSize;

	/** Set to alignment of allocation that triggered out of memory, zero otherwise. */
	static CORE_API uint32 OOMAllocationAlignment;

	/** Preallocated buffer to delete on out of memory. Used by OOM handling and crash reporting. */
	static CORE_API void* BackupOOMMemoryPool;

	/** Size of BackupOOMMemoryPool in bytes. */
	static CORE_API uint32 BackupOOMMemoryPoolSize;

	/**
	 * Various memory regions that can be used with memory stats. The exact meaning of
	 * the enums are relatively platform-dependent, although the general ones (Physical, GPU)
	 * are straightforward. A platform can add more of these, and it won't affect other 
	 * platforms, other than a minuscule amount of memory for the StatManager to track the
	 * max available memory for each region (uses an array FPlatformMemory::MCR_MAX big)
	 */
	enum EMemoryCounterRegion
	{
		MCR_Invalid, // not memory
		MCR_Physical, // main system memory
		MCR_GPU, // memory directly a GPU (graphics card, etc)
		MCR_GPUSystem, // system memory directly accessible by a GPU
		MCR_TexturePool, // presized texture pools
		MCR_StreamingPool, // amount of texture pool available for streaming.
		MCR_UsedStreamingPool, // amount of texture pool used for streaming.
		MCR_GPUDefragPool, // presized pool of memory that can be defragmented.
		MCR_PhysicalLLM, // total physical memory including CPU and GPU
		MCR_MAX
	};

	/** Which allocator is being used */
	enum EMemoryAllocatorToUse
	{
		Ansi, // Default C allocator
		Stomp, // Allocator to check for memory stomping
		TBB, // Thread Building Blocks malloc
		Jemalloc, // Linux/FreeBSD malloc
		Binned, // Older binned malloc
		Binned2, // Newer binned malloc
		Binned3, // Newer VM-based binned malloc, 64 bit only
		Binned4, // New experimental binned allocator
		Platform, // Custom platform specific allocator
		Mimalloc, // mimalloc
		Libpas, // libpas
	};

	/** Current allocator */
	static CORE_API EMemoryAllocatorToUse AllocatorToUse;

	/**
	 * Flags used for shared memory creation/open
	 */
	enum ESharedMemoryAccess
	{
		Read	=		(1 << 1),
		Write	=		(1 << 2)
	};

	/**
	 * Generic representation of a shared memory region
	 */
	struct FSharedMemoryRegion
	{
		/** Returns the name of the region */
		const TCHAR *	GetName() const			{ return Name; }

		/** Returns the beginning of the region in process address space */
		void *			GetAddress()			{ return Address; }

		/** Returns the beginning of the region in process address space */
		const void *	GetAddress() const		{ return Address; }

		/** Returns size of the region in bytes */
		SIZE_T			GetSize() const			{ return Size; }
	
		
		FSharedMemoryRegion(const FString& InName, uint32 InAccessMode, void* InAddress, SIZE_T InSize);

	protected:

		enum Limits
		{
			MaxSharedMemoryName		=	128
		};

		/** Name of the region */
		TCHAR			Name[MaxSharedMemoryName];

		/** Access mode for the region */
		uint32			AccessMode;

		/** The actual buffer */
		void *			Address;

		/** Size of the buffer */
		SIZE_T			Size;
	};



	/** Initializes platform memory specific constants. */
	static CORE_API void Init();
	
	[[noreturn]] static CORE_API void OnOutOfMemory(uint64 Size, uint32 Alignment);

	/** Initializes the memory pools, should be called by the init function. */
	static CORE_API void SetupMemoryPools();

	
	/**
	 * @return how much memory the platform should pre-allocate for crash handling (this will be allocated ahead of time, and freed when system runs out of memory).
	 */
	static uint32 GetBackMemoryPoolSize()
	{
		// by default, don't create a backup memory buffer
		return 0;
	}

	/**
	 * @return the default allocator.
	 */
	static CORE_API FMalloc* BaseAllocator();

	/**
	 * @return platform specific current memory statistics. Note: On some platforms, unused allocator cached memory is taken into account in AvailablePhysical. 
	 */
	static CORE_API FPlatformMemoryStats GetStats();

	/**
	 * @return platform specific raw stats.
	 */
	static CORE_API FPlatformMemoryStats GetStatsRaw();

	/**
	* @return memory used for platforms that can do it quickly (without affecting stat unit much)
	*/
	static CORE_API uint64 GetMemoryUsedFast();

	/**
	 * Writes all platform specific current memory statistics in the format usable by the malloc profiler.
	 */
	static CORE_API void GetStatsForMallocProfiler( FGenericMemoryStats& out_Stats );

	/**
	 * @return platform specific memory constants.
	 */
	static CORE_API const FPlatformMemoryConstants& GetConstants();
	
	/**
	 * @return approximate physical RAM in GB.
	 */
	static CORE_API uint32 GetPhysicalGBRam();

	/**
	 * Changes the protection on a region of committed pages in the virtual address space.
	 *
	 * @param Ptr Address to the starting page of the region of pages whose access protection attributes are to be changed.
	 * @param Size The size of the region whose access protection attributes are to be changed, in bytes.
	 * @param bCanRead Can the memory be read.
	 * @param bCanWrite Can the memory be written to.
	 * @return True if the specified pages' protection mode was changed.
	 */
	static CORE_API bool PageProtect(void* const Ptr, const SIZE_T Size, const bool bCanRead, const bool bCanWrite);

	/**
	 * Allocates pages from the OS.
	 *
	 * @param Size Size to allocate, not necessarily aligned
	 *
	 * @return OS allocated pointer for use by binned allocator
	 */
	static CORE_API void* BinnedAllocFromOS( SIZE_T Size );
	
	/**
	 * Returns pages allocated by BinnedAllocFromOS to the OS.
	 *
	 * @param A pointer previously returned from BinnedAllocFromOS
	 * @param Size size of the allocation previously passed to BinnedAllocFromOS
	 */
	static CORE_API void BinnedFreeToOS( void* Ptr, SIZE_T Size );
	
	/**
	 * Performs initial setup for MiMalloc.
	 * This is a noop on platforms that do not support MiMalloc, or when MIMALLOC_ENABLED is not defined.
	 */
	static void MiMallocInit()
	{
	}

	/**
	 *	Performs initial setup for Nano malloc.
	 *	This is a noop on non-apple platforms
	 */
	static void NanoMallocInit()
	{
		return;
	}
	
	/**
	 * Was this pointer allocated by the OS malloc?
	 * Currently only Apple platforms implement this to detect small block allocations.
	 *
	 * @param The pointer to query
	 * @return True if this pointer was allocated by the OS.
	 *
	 */
	static bool PtrIsOSMalloc( void* Ptr)
	{
		return false;
	}
	
	/**
	 *
	 * Nano Malloc is Apple's tiny block allocator.
	 * Does the Nano malloc zone exist?
	 *
	 * @return True if Nano malloc is enabled and available.
	*
	*/
	static bool IsNanoMallocAvailable()
	{
		return false;
	}
	
	/**
	* Was this pointer allocated by in the Nano Malloc Zone?
	* Currently only Apple platforms implement this to detect small block allocations.
	*
	* @param The pointer to query
	* @return True if this pointer is in the Nano Malloc Region
	*
	*/
	static bool PtrIsFromNanoMalloc( void* Ptr)
	{
		return false;
	}

	class FBasicVirtualMemoryBlock
	{
	protected:
		void *Ptr;
		uint32 VMSizeDivVirtualSizeAlignment;

	public:

		FBasicVirtualMemoryBlock()
			: Ptr(nullptr)
			, VMSizeDivVirtualSizeAlignment(0)
		{
		}

		FBasicVirtualMemoryBlock(void *InPtr, uint32 InVMSizeDivVirtualSizeAlignment)
			: Ptr(InPtr)
			, VMSizeDivVirtualSizeAlignment(InVMSizeDivVirtualSizeAlignment)
		{
		}

		FBasicVirtualMemoryBlock(const FBasicVirtualMemoryBlock& Other) = default;
		FBasicVirtualMemoryBlock& operator=(const FBasicVirtualMemoryBlock& Other) = default;

		UE_FORCEINLINE_HINT uint32 GetActualSizeInPages() const
		{
			return VMSizeDivVirtualSizeAlignment;
		}

		UE_FORCEINLINE_HINT void* GetVirtualPointer() const
		{
			return Ptr;
		}

		
#if 0 // documentation, must be defined by the platform
		void Commit(size_t InOffset, size_t InSize);
		void Decommit(size_t InOffset, size_t InSize);
		void FreeVirtual();

		UE_FORCEINLINE_HINT void CommitByPtr(void *InPtr, size_t InSize)
		{
			Commit(size_t(((uint8*)InPtr) - ((uint8*)Ptr)), InSize);
		}

		UE_FORCEINLINE_HINT void DecommitByPtr(void *InPtr, size_t InSize)
		{
			Decommit(size_t(((uint8*)InPtr) - ((uint8*)Ptr)), InSize);
		}

		UE_FORCEINLINE_HINT void Commit()
		{
			Commit(0, GetActualSize());
		}

		UE_FORCEINLINE_HINT void Decommit()
		{
			Decommit(0, GetActualSize());
		}

		UE_FORCEINLINE_HINT size_t GetActualSize() const
		{
			return VMSizeDivVirtualSizeAlignment * GetVirtualSizeAlignment();
		}

		/**
		 * Returns uncommitted virtual memory
		 *
		 * @param Size Size of virtual memory block, must be aligned to GetVirtualSizeAlignment
		 * @param InAlignment Alignment of returned virtual Ptr
		 */
		static FPlatformVirtualMemoryBlock AllocateVirtual(size_t Size, size_t InAlignment = FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment());
		static size_t GetCommitAlignment();
		static size_t GetVirtualSizeAlignment();
#endif
	};

	/**
	 * Some platforms may pool allocations of this size to reduce OS calls. This function
	 * serves as a hint for BinnedMalloc's CachedOSPageAllocator so it does not cache these allocations additionally
	 */
	static bool BinnedPlatformHasMemoryPoolForThisSize(SIZE_T Size)
	{
		return false;
	}

	/** Dumps basic platform memory statistics into the specified output device. */
	static CORE_API void DumpStats( FOutputDevice& Ar );

	/** Dumps basic platform memory statistics and allocator specific statistics into the specified output device. */
	static CORE_API void DumpPlatformAndAllocatorStats( FOutputDevice& Ar );

	/**
	 * Return which "high level", per platform, memory bucket we are in
	 */
	static CORE_API EPlatformMemorySizeBucket GetMemorySizeBucket();

	/** @name Memory functions */

	/** Copies count bytes of characters from Src to Dest. If some regions of the source
	 * area and the destination overlap, memmove ensures that the original source bytes
	 * in the overlapping region are copied before being overwritten.  NOTE: make sure
	 * that the destination buffer is the same size or larger than the source buffer!
	 */
	static UE_FORCEINLINE_HINT void* Memmove( void* Dest, const void* Src, SIZE_T Count )
	{
#if UE_PLATFORM_REQUIRES_NONNULL_MEMOP_ARGS
		// On this platform `memmove` is required to not have `nullptr` as `Dest` or `Src`.
		// 
		// This will eventually go away when `n3322` is adopted, but for now we workaround
		// this by checking for `Count == 0` and bailing early (the only defined behavior
		// way going forward).
		if (UNLIKELY(Count == 0))
		{
			return Dest;
		}
#endif

		return memmove( Dest, Src, Count );
	}

	static UE_FORCEINLINE_HINT int32 Memcmp( const void* Buf1, const void* Buf2, SIZE_T Count )
	{
#if UE_PLATFORM_REQUIRES_NONNULL_MEMOP_ARGS
		// On this platform `memcmp` is required to not have `nullptr` as `Dest` or `Src`.
		// 
		// This will eventually go away when `n3322` is adopted, but for now we workaround
		// this by checking for `Count == 0` and bailing early (the only defined behavior
		// way going forward).
		if (UNLIKELY(Count == 0))
		{
			return 0;
		}
#endif

		return memcmp( Buf1, Buf2, Count );
	}

	static UE_FORCEINLINE_HINT void* Memset(void* Dest, uint8 Char, SIZE_T Count)
	{
#if UE_PLATFORM_REQUIRES_NONNULL_MEMOP_ARGS
		// On this platform `memset` is required to not have `nullptr` as `Dest`.
		//
		// This will eventually go away when `n3322` is adopted, but for now we workaround
		// this by checking for `Count == 0` and bailing early (the only defined behavior
		// way going forward).
		if (UNLIKELY(Count == 0))
		{
			return Dest;
		}
#endif

		return memset( Dest, Char, Count );
	}

	static UE_FORCEINLINE_HINT void* Memzero(void* Dest, SIZE_T Count)
	{
#if UE_PLATFORM_REQUIRES_NONNULL_MEMOP_ARGS
		// On this platform `memset` is required to not have `nullptr` as `Dest`.
		//
		// This will eventually go away when `n3322` is adopted, but for now we workaround
		// this by checking for `Count == 0` and bailing early (the only defined behavior
		// way going forward).
		if (UNLIKELY(Count == 0))
		{
			return Dest;
		}
#endif

		return memset( Dest, 0, Count );
	}

	static UE_FORCEINLINE_HINT void* Memcpy(void* Dest, const void* Src, SIZE_T Count)
	{
#if UE_PLATFORM_REQUIRES_NONNULL_MEMOP_ARGS
		// On this platform `memcpy` is required to not have `nullptr` as `Dest` or `Src`.
		// 
		// This will eventually go away when `n3322` is adopted, but for now we workaround
		// this by checking for `Count == 0` and bailing early (the only defined behavior
		// way going forward).
		if (UNLIKELY(Count == 0))
		{
			return Dest;
		}
#endif

		return memcpy( Dest, Src, Count );
	}

	/** Memcpy optimized for big blocks. */
	static UE_FORCEINLINE_HINT void* BigBlockMemcpy(void* Dest, const void* Src, SIZE_T Count)
	{
#if UE_PLATFORM_REQUIRES_NONNULL_MEMOP_ARGS
		// On this platform `memcpy` is required to not have `nullptr` as `Dest` or `Src`.
		// 
		// This will eventually go away when `n3322` is adopted, but for now we workaround
		// this by checking for `Count == 0` and bailing early (the only defined behavior
		// way going forward).
		if (UNLIKELY(Count == 0))
		{
			return Dest;
		}
#endif

		return memcpy(Dest, Src, Count);
	}

	/** On some platforms memcpy optimized for big blocks that avoid L2 cache pollution are available */
	static UE_FORCEINLINE_HINT void* StreamingMemcpy(void* Dest, const void* Src, SIZE_T Count)
	{
#if UE_PLATFORM_REQUIRES_NONNULL_MEMOP_ARGS
		// On this platform `memcpy` is required to not have `nullptr` as `Dest` or `Src`.
		// 
		// This will eventually go away when `n3322` is adopted, but for now we workaround
		// this by checking for `Count == 0` and bailing early (the only defined behavior
		// way going forward).
		if (UNLIKELY(Count == 0))
		{
			return Dest;
		}
#endif

		return memcpy( Dest, Src, Count );
	}

	/** On some platforms memcpy can be distributed over multiple threads for throughput. */
	static inline void* ParallelMemcpy(void* Dest, const void* Src, SIZE_T Count, EMemcpyCachePolicy Policy = EMemcpyCachePolicy::StoreCached)
	{
#if UE_PLATFORM_REQUIRES_NONNULL_MEMOP_ARGS
		// On this platform `memcpy` is required to not have `nullptr` as `Dest` or `Src`.
		// 
		// This will eventually go away when `n3322` is adopted, but for now we workaround
		// this by checking for `Count == 0` and bailing early (the only defined behavior
		// way going forward).
		if (UNLIKELY(Count == 0))
		{
			return Dest;
		}
#endif

		(void)Policy;
		return memcpy( Dest, Src, Count );
	}

private:
	template <typename T>
	static inline void Valswap(T& A, T& B)
	{
		// Usually such an implementation would use move semantics, but
		// we're only ever going to call it on fundamental types and MoveTemp
		// is not necessarily in scope here anyway, so we don't want to
		// #include it if we don't need to.
		T Tmp = A;
		A = B;
		B = Tmp;
	}

	static CORE_API void MemswapGreaterThan8( void* Ptr1, void* Ptr2, SIZE_T Size );

public:
	static inline void Memswap( void* Ptr1, void* Ptr2, SIZE_T Size )
	{
		switch (Size)
		{
			case 0:
				break;

			case 1:
				Valswap(*(uint8*)Ptr1, *(uint8*)Ptr2);
				break;

			case 2:
				Valswap(*(uint16*)Ptr1, *(uint16*)Ptr2);
				break;

			case 3:
				Valswap(*((uint16*&)Ptr1)++, *((uint16*&)Ptr2)++);
				Valswap(*(uint8*)Ptr1, *(uint8*)Ptr2);
				break;

			case 4:
				Valswap(*(uint32*)Ptr1, *(uint32*)Ptr2);
				break;

			case 5:
				Valswap(*((uint32*&)Ptr1)++, *((uint32*&)Ptr2)++);
				Valswap(*(uint8*)Ptr1, *(uint8*)Ptr2);
				break;

			case 6:
				Valswap(*((uint32*&)Ptr1)++, *((uint32*&)Ptr2)++);
				Valswap(*(uint16*)Ptr1, *(uint16*)Ptr2);
				break;

			case 7:
				Valswap(*((uint32*&)Ptr1)++, *((uint32*&)Ptr2)++);
				Valswap(*((uint16*&)Ptr1)++, *((uint16*&)Ptr2)++);
				Valswap(*(uint8*)Ptr1, *(uint8*)Ptr2);
				break;

			case 8:
				Valswap(*(uint64*)Ptr1, *(uint64*)Ptr2);
				break;

			case 16:
				Valswap(((uint64*)Ptr1)[0], ((uint64*)Ptr2)[0]);
				Valswap(((uint64*)Ptr1)[1], ((uint64*)Ptr2)[1]);
				break;

			default:
				MemswapGreaterThan8(Ptr1, Ptr2, Size);
				break;
		}
	}

	/**
	* Loads a simple POD type from unaligned memory.
	*
	* @param Ptr unaligned memory of at least size sizeof(T)
	* @return Value at Ptr
	*/
	template <typename T>
	static inline T ReadUnaligned(const void* Ptr)
	{
		T AlignedT;
		memcpy((void*)&AlignedT, Ptr, sizeof(T));
		return AlignedT;
	}

	/**
	* Stores a simple POD type to unaligned memory.
	*
	* @param Ptr unaligned memory of at least size sizeof(T)
	* @param Value value to write at Ptr
	*/
	template <typename T>
	static UE_FORCEINLINE_HINT void WriteUnaligned(void* Ptr, const T& InValue)
	{
		memcpy(Ptr, &InValue, sizeof(T));
	}

	/**
	 * Maps a named shared memory region into process address space (creates or opens it)
	 *
	 * @param Name unique name of the shared memory region (should not contain [back]slashes to remain cross-platform).
	 * @param bCreate whether we're creating it or just opening existing (created by some other process).
	 * @param AccessMode mode which we will be accessing it (use values from ESharedMemoryAccess)
	 * @param Size size of the buffer (should be >0. Also, the real size is subject to platform limitations and may be increased to match page size)
	 *
	 * @return pointer to FSharedMemoryRegion (or its descendants) if successful, NULL if not.
	 */
	static CORE_API FSharedMemoryRegion* MapNamedSharedMemoryRegion(const FString& Name, bool bCreate, uint32 AccessMode, SIZE_T Size);

	/**
	 * Unmaps a name shared memory region
	 *
	 * @param MemoryRegion an object that encapsulates a shared memory region (will be destroyed even if function fails!)
	 *
	 * @return true if successful
	 */
	static CORE_API bool UnmapNamedSharedMemoryRegion(FSharedMemoryRegion * MemoryRegion);

	/**
	*	Gets whether this platform supports Fast VRAM memory
	*		Ie, whether TexCreate_FastVRAM flags actually mean something or not
	*
	*	@return	bool		true if supported, false if not
	*/
	static UE_FORCEINLINE_HINT bool SupportsFastVRAMMemory()
	{
		return false;
	}

	/**
	* Returns true if debug memory has been assigned to the title for general use.
	* Only applies to platforms with fixed memory and no paging.
	*/
	static CORE_API bool IsExtraDevelopmentMemoryAvailable();

	/**
	* Returns >0 if debug memory has been assigned to the title for general use.
	* Only applies to platforms with fixed memory and no paging.
	*/
	static CORE_API uint64 GetExtraDevelopmentMemorySize();

	/**
	* Returns initial program size or 0 if the platform doesn't track initial program size
	*/
	static uint64 GetProgramSize()
	{
		return ProgramSize;
	}

	/**
	* Sets the initial program size
	*/
	static void SetProgramSize(uint64 InProgramSize)
	{
		ProgramSize = InProgramSize;
	}

	/**
	* This function sets AllocFunction and FreeFunction and returns true, or just returns false.
	* These functions are the platform dependant low low low level functions that LLM uses to allocate memory.
	*/
	static CORE_API bool GetLLMAllocFunctions(void*(*&OutAllocFunction)(size_t), void(*&OutFreeFunction)(void*, size_t), int32& OutAlignment);

	/**
	* Called for all default tracker LLM allocations and frees, when LLM is enabled.
	* Provides a single alloc/free hook that platforms can implement to support platform specific memory analysis tools.
	*/
	UE_FORCEINLINE_HINT static void OnLowLevelMemory_Alloc(void const* Pointer, uint64 Size, uint64 Tag) { }
	UE_FORCEINLINE_HINT static void OnLowLevelMemory_Free(void const* Pointer, uint64 Size, uint64 Tag) { }

	/**
	* Called once at LLM initialization time to let the platform add any custom tags
	*/
	static void RegisterCustomLLMTags() { };

	/**
	* Called once per frame when LLM is collating the data for the current frame.
	* Can be used to set platform-specific calculated tag data via SetTagAmountForTracker
	*/
	static void UpdateCustomLLMTags() { };

	/**
	 * Indicates whether LLM allocations are already accounted for with tracking and in GetStats.
	 * Returns true if LLM allocations come from a memory pool separate from the main engine's memory or
	 * are already tagged with tracking by the platform memory system.
	 * Returns false if LLM uses the regular memory shared with the engine and allocations are not tracked.
	 * @see GetLLMAllocFunctions
	 */
	UE_DEPRECATED(5.8, "This workaround is now obsolete")
	static bool TracksLLMAllocations() { return false; }

	/**
	* Returns true if Protecting the parent processes pages has been enabled
	* Only supported on platforms that support forking
	*/
	static bool HasForkPageProtectorEnabled() { return false; }

	/**
	* Return the page allocations from the operating system (/proc/self/smaps). This only means something on
	* platforms that can fork and have Copy On Write behavior. 
	*/
	static bool GetForkedPageAllocationInfo(TArray<FForkedPageAllocation>& OutPageAllocationInfos)
	{
		return false; // Most platform do not implement this.
	}

	/**
	 * Returns a pretty-string for an amount of memory given in bytes.
	 *
	 * @param Memory amount in bytes
	 * @return Memory in a pretty formatted string
	 */
	static CORE_API FString PrettyMemory( uint64 Memory );

	/**
	* Return true if the platform can allocate a lot more virtual memory than physical memory
	* It's true for most platforms, but iOS needs a special entitlement and Linux a kernel config for this
	*/
	UE_FORCEINLINE_HINT static bool CanOverallocateVirtualMemory()
	{
		return true;
	}

	// This bit is always zero in user mode addresses and most likely won't be used by current or future
	// CPU features like ARM's PAC / Top-Byte Ignore or Intel's Linear Address Masking / 5-Level Paging
#if defined(__x86_64__) || defined(_M_X64)
	static constexpr uint32 KernelAddressBit = 63;
#elif defined(__aarch64__) || defined(_M_ARM64)
	static constexpr uint32 KernelAddressBit = 55;
#endif

protected:
	friend struct FGenericStatsUpdater;

	/** Updates platform specific stats. This method is called through FGenericStatsUpdater from the task graph thread. */
	static CORE_API void InternalUpdateStats( const FPlatformMemoryStats& MemoryStats );

	/** Program memory allocation in bytes. */
	static CORE_API uint64 ProgramSize;
};
