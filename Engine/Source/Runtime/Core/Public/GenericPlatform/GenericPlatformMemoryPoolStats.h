// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	GenericPlatformMemoryPoolStats.h: Stat definitions for generic memory pools
==============================================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Stats/Stats.h"

DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Physical Memory Pool [Physical]"), MCR_Physical, STATGROUP_Memory,  FPlatformMemory::MCR_Physical, CORE_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Total Physical Memory Pool (CPU + GPU) [PhysicalLLM]"), MCR_PhysicalLLM, STATGROUP_Memory,  FPlatformMemory::MCR_PhysicalLLM, CORE_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("GPU Memory Pool [GPU]"), MCR_GPU, STATGROUP_Memory,  FPlatformMemory::MCR_GPU, CORE_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Texture Memory Pool [Texture]"), MCR_TexturePool, STATGROUP_Memory,  FPlatformMemory::MCR_TexturePool, CORE_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Streaming Texture Pool [Streaming]"), MCR_StreamingPool, STATGROUP_Memory,  FPlatformMemory::MCR_StreamingPool, CORE_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Used Streaming Pool [Wanted]"), MCR_UsedStreamingPool, STATGROUP_Memory,  FPlatformMemory::MCR_UsedStreamingPool, CORE_API);

// Must match values in the MemoryProfiler2.FMemoryAllocationStatsV4
DECLARE_MEMORY_STAT_EXTERN(TEXT("Total Physical"),		STAT_TotalPhysical,STATGROUP_MemoryPlatform, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Total Virtual"),		STAT_TotalVirtual,STATGROUP_MemoryPlatform, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Page Size"),			STAT_PageSize,STATGROUP_MemoryPlatform, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Total Physical GB"),	STAT_TotalPhysicalGB,STATGROUP_MemoryPlatform, CORE_API);

DECLARE_MEMORY_STAT_EXTERN(TEXT("Available Physical"),	STAT_AvailablePhysical,STATGROUP_MemoryPlatform, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Available Virtual"),	STAT_AvailableVirtual,STATGROUP_MemoryPlatform, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Used Physical"),		STAT_UsedPhysical,STATGROUP_MemoryPlatform, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Peak Used Physical"),	STAT_PeakUsedPhysical,STATGROUP_MemoryPlatform, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Used Virtual"),		STAT_UsedVirtual,STATGROUP_MemoryPlatform, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Peak Used Virtual"),	STAT_PeakUsedVirtual,STATGROUP_MemoryPlatform, CORE_API);



