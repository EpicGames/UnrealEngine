// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "CoreFwd.h"
#include "CoreTypes.h"
#include "FramePro/FrameProConfig.h"
#include "HAL/PlatformCrt.h"
#include "Math/NumericLimits.h"
#include "Misc/CompressionFlags.h"
#include "Misc/EnumClassFlags.h"
#include "ProfilingDebugging/CsvProfilerConfig.h"
#include "ProfilingDebugging/ExternalProfilerConfig.h"
#include "UObject/CoreReflectedTypeAccessors.h"

#if PLATFORM_CPU_X86_FAMILY
#include <xmmintrin.h> // _mm_prefetch
#elif PLATFORM_CPU_ARM_FAMILY && defined(_MSC_VER)
#include <intrin.h> // __prefetch
#endif

class Error;
class FDelegateHandle;
class FOutputDevice;
class FString;
class FText;
class GenericApplication;
class IInstallBundleManager;
class IPlatformChunkInstall;
class IPlatformCompression;
class IWrappedFileHandle;
class IPlatformHostCommunication;
struct FDefaultDelegateUserPolicy;
struct FGenericCrashContext;
struct FGenericMemoryWarningContext;
struct FGuid;

template <typename FuncType>
class TFunction;

template <typename FuncType, typename UserPolicy>
class TDelegate;

#if UE_BUILD_SHIPPING && !WITH_EDITOR
#define UE_DEBUG_BREAK() ((void)0)
#else
#define UE_DEBUG_BREAK() ((void)(FPlatformMisc::IsDebuggerPresent() && ([] () { UE_DEBUG_BREAK_IMPL(); } (), 1)))
#endif

// Enable this manually for debugging
#define LOG_NAMED_EVENTS 0

#define ENABLE_GENERIC_NAMED_EVENTS ((UE_EXTERNAL_PROFILING_ENABLED) || (STATS) || (ENABLE_STATNAMEDEVENTS) || ((CSV_PROFILER_STATS) && !UE_BUILD_SHIPPING) || (LOG_NAMED_EVENTS) || (FRAMEPRO_ENABLED))

/**
 * Available build configurations. Mirorred from UnrealTargetConfiguration.
 */
enum class EBuildConfiguration : uint8
{
	/** Unknown build configuration. */
	Unknown,

	/** Debug build. */
	Debug,

	/** DebugGame build. */
	DebugGame,

	/** Development build. */
	Development,

	/** Shipping build. */
	Shipping,

	/** Test build. */
	Test
};

/**
 * Controls behaviour for built in crash handling
 */
enum class ECrashHandlingType : uint8
{
	Default,	/* UE default handling */
	Disabled,	/* UE doesn't handle crashes on main thread, general workers, rendering thread. */
};

/**
 * Returns the string representation of the specified EBuildConfiguration value.
 *
 * @param Configuration The string to get the EBuildConfiguration for.
 * @return An EBuildConfiguration value.
 */
CORE_API bool LexTryParseString(EBuildConfiguration& OutConfiguration, const TCHAR* Configuration);

/**
 * Returns the string representation of the specified EBuildConfiguration value.
 *
 * @param Configuration The value to get the string for.
 * @return The string representation.
 */
CORE_API const TCHAR* LexToString(EBuildConfiguration Configuration);

namespace EBuildConfigurations
{
	/**
	 * Returns the localized text representation of the specified EBuildConfiguration value.
	 *
	 * @param Configuration The value to get the text for.
	 * @return The localized Build configuration text
	 */
	CORE_API FText ToText( EBuildConfiguration Configuration );
}

/**
 * Enumerates build target types.
 */
enum class EBuildTargetType : uint8
{
	/** Unknown build target. */
	Unknown,

	/** Game target. */
	Game,

	/** Server target. */
	Server,

	/** Client target. */
	Client,

	/** Editor target. */
	Editor,

	/** Program target. */
	Program,
};

/**
 * Returns the string representation of the specified EBuildTarget value.
 *
 * @param OutType The value to get the string for.
 * @param Text The text to parse.
 * @return The string representation.
 */
CORE_API bool LexTryParseString(EBuildTargetType& OutType, const TCHAR* Text);

/**
 * Returns the string representation of the specified EBuildTargetType value.
 *
 * @param Target The string to get the EBuildTargetType for.
 * @return An EBuildTarget::Type value.
 */
CORE_API const TCHAR* LexToString(EBuildTargetType Type);

namespace EBuildTargets
{
	UE_DEPRECATED(4.24, "EBuildTargets::Type is deprecated. Use EBuildTargetType instead.")
	typedef EBuildTargetType Type;

	UE_DEPRECATED(4.24, "EBuildTargets::Unknown is deprecated. Use EBuildTargetType::Unknown instead.")
	static constexpr EBuildTargetType Unknown = EBuildTargetType::Unknown;

	UE_DEPRECATED(4.24, "EBuildTargets::Editor is deprecated. Use EBuildTargetType::Unknown instead.")
	static constexpr EBuildTargetType Editor = EBuildTargetType::Editor;

	UE_DEPRECATED(4.24, "EBuildTargets::Game is deprecated. Use EBuildTargetType::Unknown instead.")
	static constexpr EBuildTargetType Game = EBuildTargetType::Game;

	UE_DEPRECATED(4.24, "EBuildTargets::Server is deprecated. Use EBuildTargetType::Unknown instead.")
	static constexpr EBuildTargetType Server = EBuildTargetType::Server;

	UE_DEPRECATED(4.24, "EBuildTargets::FromString is deprecated. Use LexFromString() instead.")
	CORE_API EBuildTargetType FromString( const FString& Target );

	UE_DEPRECATED(4.24, "EBuildTargets::FromString is deprecated. Use LexFromString() instead.")
	CORE_API const TCHAR* ToString(EBuildTargetType Target);
}

/**
 * Enumerates the modes a convertible laptop can be in.
 */
enum class EConvertibleLaptopMode
{
	/** Not a convertible laptop. */
	NotSupported,

	/** Laptop arranged as a laptop. */
	Laptop,

	/** Laptop arranged as a tablet. */
	Tablet
};

/** Device orientations for screens. e.g. Landscape, Portrait, etc.*/
enum class EDeviceScreenOrientation : uint8
{
	/** The orientation is not known */
	Unknown,

	/** The orientation is portrait with the home button at the bottom */
	Portrait,

	/** The orientation is portrait with the home button at the top */
	PortraitUpsideDown,

	/** The orientation is landscape with the home button at the right side */
	LandscapeLeft,

	/** The orientation is landscape with the home button at the left side */
	LandscapeRight,

	/** The orientation is as if place on a desk with the screen upward */
	FaceUp,

	/** The orientation is as if place on a desk with the screen downward */
	FaceDown,

	/** The orientation is portrait, oriented upright with the sensor */
	PortraitSensor,

	/** The orientation is landscape, oriented upright with the sensor */
	LandscapeSensor,

	/** The orientation is no longer locked and adjusts according to the sensor */
	FullSensor,
};


namespace EErrorReportMode
{
	/**
	 * Enumerates supported error reporting modes.
	 */
	enum Type
	{
		/** Displays a call stack with an interactive dialog for entering repro steps, etc. */
		Interactive,

		/** Unattended mode.  No repro steps, just submits data straight to the server */
		Unattended,

		/** Same as unattended, but displays a balloon window in the system tray to let the user know */
		Balloon,
	};
}


/**
 * Enumerates supported message dialog category types.
 */
enum class EAppMsgCategory : uint8
{
	Warning,
	Error,
	Success,
	Info,
};

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<EAppMsgCategory>();
#endif


namespace EAppMsgType
{
	/**
	 * Enumerates supported message dialog button types.
	 */
	enum Type : int
	{
		Ok,
		YesNo,
		OkCancel,
		YesNoCancel,
		CancelRetryContinue,
		YesNoYesAllNoAll,
		YesNoYesAllNoAllCancel,
		YesNoYesAll,
	};
}

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<EAppMsgType::Type>();
#endif


namespace EAppReturnType
{
	/**
	 * Enumerates message dialog return types.
	 */
	enum Type : int
	{
		No,
		Yes,
		YesAll,
		NoAll,
		Cancel,
		Ok,
		Retry,
		Continue,
	};
}

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<EAppReturnType::Type>();
#endif

/**
 * Returns the string representation of the specified EAppReturnType::Type value.
 *
 * @param Value The value to get the string for.
 * @return The string representation.
 */
CORE_API const TCHAR* LexToString(EAppReturnType::Type Value);

/*
 * Holds a computed SHA256 hash.
 */
struct FSHA256Signature
{
	uint8 Signature[32];

	/** Generates a hex string of the signature */
	CORE_API FString ToString() const;
};

enum class EMobileHapticsType : uint8
{
	// these are IOS UIFeedbackGenerator types
	FeedbackSuccess,
	FeedbackWarning,
	FeedbackError,
	SelectionChanged,
	ImpactLight,
	ImpactMedium,
	ImpactHeavy,
};

/** Possible connection states */
enum class ENetworkConnectionStatus : uint8
{
	/** Default state */
	Unknown = 0,
	/** No network connection or network device disabled */
	Disabled,
	/** Ad-hoc Wifi network or LAN with no external connection */
	Local,
	/** Connected to the network */
	Connected
};
CORE_API const TCHAR* LexToString(ENetworkConnectionStatus EnumVal);

enum class ENetworkConnectionType : uint8
{
	/**
	 * Enumerates the network connection types
	 */
	Unknown,
	None,
	AirplaneMode,
	Cell,
	WiFi,
	WiMAX,
	Bluetooth,
	Ethernet,
};

typedef TDelegate<void(ENetworkConnectionType ConnectionType), FDefaultDelegateUserPolicy> FOnNetworkConnectionChangedDelegate;

enum class EProcessDiagnosticFlags : uint32
{
	None            = 0,
	MemorySanitizer = 1 << 0,
	AnsiMalloc      = 1 << 1,
	StompMalloc     = 1 << 2,
};

ENUM_CLASS_FLAGS(EProcessDiagnosticFlags);

/**
 * Returns the string representation of the specified ENetworkConnection value.
 *
 * @param Target The value to get the string for.
 * @return The string representation.
 */
CORE_API const TCHAR* LexToString( ENetworkConnectionType Target );

struct FProcessorGroupDesc 
{
	static constexpr uint16 MaxNumProcessorGroups = 16;
	uint64 ThreadAffinities[MaxNumProcessorGroups] = {}; 
	uint16 NumProcessorGroups = 0;
};

/**
 * Different types of Page Fault stats
 */
struct FPageFaultStats
{
	/** Page faults for data already in memory (platform dependent) */
	uint64 SoftPageFaults	= 0;

	/** Page faults for data on disk (platform dependent) */
	uint64 HardPageFaults	= 0;

	/** All page fault types */
	uint64 TotalPageFaults	= 0;
};

/**
 * Flags for page fault stats to retrieve
 */
enum class EPageFaultFlags : uint8
{
	SoftPageFaults		= 0x01,
	HardPageFaults		= 0x02,
	TotalPageFaults		= 0x04,

	All					= 0xFF
};

ENUM_CLASS_FLAGS(EPageFaultFlags);


/**
 * Different types of Input/Output stats
 */
struct FProcessIOStats
{
	/** Blocking Input operations */
	uint64 BlockingInput	= 0;

	/** Blocking Output operations */
	uint64 BlockingOutput	= 0;

	/** Blocking operations that were neither Input or Output operations (platform dependent) */
	uint64 BlockingOther	= 0;

	/** Input bytes transferred */
	uint64 InputBytes		= 0;

	/** Output bytes transferred */
	uint64 OutputBytes		= 0;

	/** Bytes transferred that were not from either Input or Output operations */
	uint64 OtherBytes		= 0;
};

/**
 * Flags for Input/Output stats to retrieve
 */
enum class EInputOutputFlags : uint8
{
	BlockingInput		= 0x01,
	BlockingOutput		= 0x02,
	BlockingOther		= 0x04,
	InputBytes			= 0x08,
	OutputBytes			= 0x10,
	OtherBytes			= 0x20,

	All					= 0xFF
};

ENUM_CLASS_FLAGS(EInputOutputFlags);

/**
 * Defines the type of format the backbuffer expects. You must update values in TonemapCommon.ush when changing this enum
 */
enum class EDisplayOutputFormat
{
	SDR_sRGB = 0,
	SDR_Rec709 = 1,
	SDR_ExplicitGammaMapping = 2,
	HDR_ACES_1000nit_ST2084 = 3,
	HDR_ACES_2000nit_ST2084 = 4,
	HDR_ACES_1000nit_ScRGB = 5,
	HDR_ACES_2000nit_ScRGB = 6,
	HDR_LinearEXR = 7,
	HDR_LinearNoToneCurve = 8,
	HDR_LinearWithToneCurve = 9,

	MAX
};

/**
 * Display gamut, format, and chromacities. You must update values in TonemapCommon.ush when changing this enum + EHDRCaptureGamut
 */
enum class EDisplayColorGamut
{
	sRGB_D65 = 0,
	DCIP3_D65 = 1,
	Rec2020_D65 = 2,
	ACES_D60 = 3,
	ACEScg_D60 = 4,

	MAX
};

/**
 * Different types of Context Switch stats
 */
struct FContextSwitchStats
{
	/** Context switches that occurred voluntarily (e.g. yield - platform dependent) */
	uint64 VoluntaryContextSwitches		= 0;

	/** Context switches that were involuntary (e.g. quantum reached/exceeded - platform dependent) */
	uint64 InvoluntaryContextSwitches	= 0;

	/** All context switch types */
	uint64 TotalContextSwitches			= 0;
};

/**
 * Flags for Context Switch stats to retrieve
 */
enum class EContextSwitchFlags : uint8
{
	VoluntaryContextSwitches		= 0x01,
	InvoluntaryContextSwitches		= 0x02,
	TotalContextSwitches			= 0x04,

	All								= 0xFF
};

ENUM_CLASS_FLAGS(EContextSwitchFlags);

/**
 * Generic Thermal State Enum returned by platform implementations of FGenericPlatformMisc::GetDeviceThermalState.
 * This is a direct mapping to Android getThermalStatus.  This also means that historical android status data matches.
 * IOS NSProcessInfoThermalState, which provides less detail, will be mapped onto the first 4 values.
 * Unless a platform provides enough to supply at least the None and Severe states GetDeviceThermalState should be left Unsupported.
 * In all cases platform specific stats may have better information.
 */
enum class EDeviceThermalState : int32
{
	Unsupported = -1,	// Feature is unsupported or an error is returned from the platform (which may also means unsupported on this device).

	None = 0,		// No thermal perf impacts.								If this function is supported on a platform this value should be meaningful.
	Light = 1,		// Device likely warm, but little or no throttling.
	Moderate = 2,	// Device likely warmer, but little or no throttling.
	Severe = 3,		// Serious throttling engaged, Device likely hot.		If this function is supported on a platform this value should be meaningful.
	Critical = 4,	// Fully throttling.
	Emergency = 5,	// Device functionality limited.
	Shutdown = 6,	// Device should shutdown.
};

namespace ECPUFeatureBits_X86
{
	inline constexpr uint32 SSE2 = 1U << 2;
	inline constexpr uint32 SSSE3 = 1U << 3;
	inline constexpr uint32 SSE42 = 1U << 4;
	inline constexpr uint32 AVX = 1U << 5;
	inline constexpr uint32 BMI1 = 1U << 6; // Bit Manipulation Instructions - 1
	inline constexpr uint32 BMI2 = 1U << 7; // Bit Manipulation Instructions - 2
	inline constexpr uint32 AVX2 = 1U << 8;
	inline constexpr uint32 F16C = 1U << 9; // Float16 conversion instructions
	inline constexpr uint32 AVX512 = 1U << 10; // Skylake feature set : AVXF512{ F,VL,BW,DQ}.
	inline constexpr uint32 AVX512_NOCAVEATS = 1U << 11; // Set when we have AVX512 without caveats like throttling.
	inline constexpr uint32 FMA = 1U << 12;
}

typedef TDelegate<void(bool), FDefaultDelegateUserPolicy> FOnAudioFocusChangedDelegate;

/**
 * Generic implementation for most platforms
 */
struct FGenericPlatformMisc
{
	/**
	 * Called during appInit() after cmd line setup
	 */
	static CORE_API void PlatformPreInit();
	static void PlatformInit() { }


	static CORE_API FDelegateHandle AddNetworkListener(FOnNetworkConnectionChangedDelegate&& InNewDelegate);
	static CORE_API bool RemoveNetworkListener(FDelegateHandle Handle);
	
	/**
	* Called to dismiss splash screen
	*/
	static void PlatformHandleSplashScreen(bool ShowSplashScreen = false) { }

	/**
	 * Called during AppExit(). Log, Config still exist at this point, but not much else does.
	 */
	static void PlatformTearDown() { }

	/** Set/restore the Console Interrupt (Control-C, Control-Break, Close) handler. */
	static void SetGracefulTerminationHandler() { }

	/**
	 * Installs handler for the unexpected (due to error) termination of the program,
	 * including, but not limited to, crashes.
	 */
	static void SetCrashHandler(void (* CrashHandler)(const FGenericCrashContext& Context)) { }

	/**
	 * Gets the current crash handling type.
	 */
	static ECrashHandlingType GetCrashHandlingType() { return ECrashHandlingType::Default; }
	
	/**
	 * Sets the type of crash handling done by the engine. Returns the new crash handling type. If disabling crash
	 * handling make sure an alternative is active since threads that crash will just disappear.
	 */
	static ECrashHandlingType SetCrashHandlingType(ECrashHandlingType Type) { return ECrashHandlingType::Default; }
	
	/**
	 * Retrieve a environment variable from the system
	 *
	 * @param VariableName The name of the variable (ie "Path")
	 * @return Value of the variable, or an empty string if not set.
	 */
	static CORE_API FString GetEnvironmentVariable(const TCHAR* VariableName);

	/**
	 * Sets an environment variable to the local process's environment
	 *
	 * @param VariableName The name of the variable (ie "Path")
	 * @param Value The string to set the variable to.
	 */
	static CORE_API void SetEnvironmentVar(const TCHAR* VariableName, const TCHAR* Value);

	/**
	 * Returns the maximum length of a path.
	 * Note that some platforms such as Windows make this a per-process configuration.
	 */
	FORCEINLINE static int32 GetMaxPathLength()
	{
		return 128;
	}

	/**
	 * Returns the maximum length of a path which applies to all processes on the host platform.
	 * Since some platforms such as Windows make the maximum path length a per-process configuration,
	 * this function returns the least commmon limit that can be supported by all processes.
	 * Default implementation returns GetMaxPathLength().
	 */
	static CORE_API int32 GetExternalAppMaxPathLength();

	/**
	 * return the delimiter between paths in the PATH environment variable.
	 */
	static CORE_API const TCHAR* GetPathVarDelimiter();

	/**
	 * Retrieve the Mac address of the current adapter. This is generally platform-dependent code that other platforms must implement.
	 *
	 * @return array of bytes representing the Mac address, or empty array if unable to determine.
	 */
	UE_DEPRECATED(4.14, "GetMacAddress is deprecated. It is not reliable on all platforms")
	static CORE_API TArray<uint8> GetMacAddress();

	/**
	 * Retrieve the Mac address of the current adapter as a string. This generally relies on GetMacAddress() being implemented for the platform, if possible.
	 *
	 * @return String representing the Mac address, or empty string.
	 */
	UE_DEPRECATED(4.14, "GetMacAddressString is deprecated. It is not reliable on all platforms")
	static CORE_API FString GetMacAddressString();

	/**
	 * Retrieve the Mac address of the current adapter as a hashed string (privacy). This generally relies on GetMacAddress() being implemented for the platform, if possible.
	 *
	 * @return String representing the hashed Mac address, or empty string.
	 */
	UE_DEPRECATED(4.14, "GetHasedMacAddressString is deprecated. It is not reliable on all platforms")
	static CORE_API FString GetHashedMacAddressString();

	/**
	 * Returns a unique string for device identification. Differs from the deprecated GetUniqueDeviceId
	 * in that there is no default implementation (which used unreliable Mac address determiniation).
	 * This code is expected to use platform-specific methods to identify the device.
	 *
	 * WARNING: Use of this method in your app may imply technical certification requirments for your platform!
	 * For instance, consoles often require cert waivers to be in place before calling APIs that can track a device,
	 * so be very careful that you are following your platform's protocols for accessing device IDs. See the platform-
	 * specific implementations of this method for details on what APIs are used.
	 *
	 * If you do not have permission to call this on one or more of your platforms, set GET_DEVICE_ID_UNAVAILABLE=1
	 * in your build step to ensure that any calls that may be made to this API will simply return an empty string.
	 *
	 * @return the unique string generated by this platform for this device, or an empty string if one is not available.
	 */
	static CORE_API FString GetDeviceId();

	/**
	 * Returns a unique string for advertising identification
	 *
	 * @return the unique string generated by this platform for this device
	 */
	static CORE_API FString GetUniqueAdvertisingId();

	// #CrashReport: 2015-02-24 Remove
	/** Submits a crash report to a central server (release builds only) */
	static CORE_API void SubmitErrorReport( const TCHAR* InErrorHist, EErrorReportMode::Type InMode );

	/** Check to see if the platform is being viewed remotely. In such a mode we should aim to minimize screen refresh to get the best performance on the remote viewer */
	static bool IsRemoteSession()
	{
		return false;
	}

	/** Return true if a debugger is present */
	FORCEINLINE static bool IsDebuggerPresent()
	{
#if UE_BUILD_SHIPPING
		return false;
#else
		return true; // unknown platforms return true so that they can crash into a debugger
#endif
	}

	/** Returns non-zero if a performance-sensitive diagnostic, like a memory sanitizer, is detected for this process */
	static CORE_API EProcessDiagnosticFlags GetProcessDiagnostics();

	/** Break into the debugger, if IsDebuggerPresent returns true, otherwise do nothing  */
	UE_DEPRECATED(4.19, "FPlatformMisc::DebugBreak is deprecated. Use the UE_DEBUG_BREAK() macro instead.")
	FORCEINLINE static void DebugBreak()
	{
		if (IsDebuggerPresent())
		{
			*((int32*)3) = 13; // unknown platforms crash into the debugger
		}
	}


	// Query the CPUID and parse out various feature bits. This is safe to call multiple times and caches the result internally for rapid access.
	// Bits are all from the ECPUFeatureBits_X86 namespace.
	static CORE_API uint32 GetFeatureBits_X86();
	static bool CheckFeatureBit_X86(uint32 FeatureBit_X86) { return (GetFeatureBits_X86() & FeatureBit_X86) != 0; }
	static bool CheckAllFeatureBits_X86(uint32 FeatureBits_X86) { return (GetFeatureBits_X86() & FeatureBits_X86) == FeatureBits_X86; }

	/**
	 * Determines if AVX2 instruction set is supported on this platform
	 *
	 * @return	Returns true if instruction-set is supported
	 */
	static CORE_API bool HasAVX2InstructionSupport();

	/**
	 * Uses cpuid instruction to get the vendor string
	 *
	 * @return	CPU vendor name
	 */
	static CORE_API FString GetCPUVendor();

	/**
	 * On x86(-64) platforms, uses cpuid instruction to get the CPU signature
	 *
	 * @return	CPU info bitfield
	 *
	 *			Bits 0-3	Stepping ID
	 *			Bits 4-7	Model
	 *			Bits 8-11	Family
	 *			Bits 12-13	Processor type (Intel) / Reserved (AMD)
	 *			Bits 14-15	Reserved
	 *			Bits 16-19	Extended model
	 *			Bits 20-27	Extended family
	 *			Bits 28-31	Reserved
	 */
	static CORE_API uint32 GetCPUInfo();

	/** @return whether this cpu supports certain required instructions or not */
	UE_DEPRECATED(5.8, "Replace HasNonoptionalCPUFeatures() with false.")
	static inline bool HasNonoptionalCPUFeatures() { return false; }
	/** @return whether to check for specific CPU compatibility or not */
	UE_DEPRECATED(5.8, "Replace NeedsNonoptionalCPUFeaturesCheck() with false.")
	static inline bool NeedsNonoptionalCPUFeaturesCheck() { return false; }

	/** @return architecture of the host machine (x64, arm64 etc) **/
	static CORE_API FString GetHostArchitecture();

	/**
	 * Uses cpuid instruction to get the CPU brand string
	 *
	 * @return	CPU brand string
	 */
	static CORE_API FString GetCPUBrand();
	static CORE_API uint32 GetCPUModelID();
	static CORE_API uint32 GetCPUMicrocodeRevision();

	/**
	 * Returns the CPU chipset if known
	 *
	 * @return	CPU chipset string (or "Unknown")
	 */
	static CORE_API FString GetCPUChipset();

	/**
	 * @return primary GPU brand string
	 */
	static CORE_API FString GetPrimaryGPUBrand();

	/**
	 * @return	"DeviceMake|DeviceModel" if possible, and "CPUVendor|CPUBrand" otherwise, optionally returns "DeviceMake|DeviceModel|CPUChipset" if known
	 */
	static CORE_API FString GetDeviceMakeAndModel();

	static CORE_API struct FGPUDriverInfo GetGPUDriverInfo(const FString& DeviceDescription);

	/**
	 * Gets the OS Version and OS Subversion.
	 */
	static CORE_API void GetOSVersions( FString& out_OSVersionLabel, FString& out_OSSubVersionLabel );

	/**
	 * Gets a string representing the numeric OS version (as opposed to a translated OS version that GetOSVersions returns).
	 * The returned string should try to be brief and avoid newlines and symbols, but there's technically no restriction on the string it can return.
	 * If the implementation does not support this, it should return an empty string.
	 */
	static CORE_API FString GetOSVersion();

	/** 
	 * Retrieves information about the total number of bytes and number of free bytes for the specified disk path. 
	 *
	 * Note that on modern file systems the concept of "free space" is a bit more loosely defined than one might historically expect due 
	 * to features like on-demand freeing of caches and automatic content deduplication. As a result, this function can be
	 * very slow (10s of ms), and the free space value should not be used for exact measurements.
	*/
	static CORE_API bool GetDiskTotalAndFreeSpace( const FString& InPath, uint64& TotalNumberOfBytes, uint64& NumberOfFreeBytes );

	/**
	 * Gets Page Fault stats covering the lifetime of the process
	 *
	 * @param OutStats	Outputs the page fault stats
	 * @param Flags		The type of page fault stats to retrieve
	 * @return			Whether or not stats were successfully retrieved
	 */
	static bool GetPageFaultStats(FPageFaultStats& OutStats, EPageFaultFlags Flags=EPageFaultFlags::All)
	{
		return false;
	}

	/**
	 * Gets Blocking Input/Output stats covering the lifetime of the process
	 *
	 * @param OutStats	Outputs the IO stats
	 * @param Flags		The type of IO stats to retrieve
	 * @return			Whether or not stats were successfully retrieved
	 */
	static bool GetBlockingIOStats(FProcessIOStats& OutStats, EInputOutputFlags Flags=EInputOutputFlags::All)
	{
		return false;
	}

	/**
	 * Gets Context Switch stats covering the lifetime of the process
	 *
	 * @param OutStats	Outputs the Context Switch stats
	 * @param Flags		The type of Context Switch stats to retrieve
	 * @return			Whether or not stats were successfully retrieved
	 */
	static bool GetContextSwitchStats(FContextSwitchStats& OutStats, EContextSwitchFlags Flags=EContextSwitchFlags::All)
	{
		return false;
	}

	static bool SupportsMessaging()
	{
		return true;
	}

	/**
	 * For platforms that support cache storage functionality, setting a directory for it when it's not available can result in errors.
	 * This method's purpose is to be overridden in such platforms, with logic that checks if said cache is available.
	 * In all other platforms without this constraint, it should return true by default.
	 */
	static bool IsCacheStorageAvailable()
	{
		return true;
	}

	static bool SupportsLocalCaching()
	{
		return true;
	}

	static bool AllowLocalCaching()
	{
#if PLATFORM_DESKTOP
		return true;
#else
		return false;
#endif
	}

	/** Platform can generate a full-memory crashdump during crash handling. */
	static bool SupportsFullCrashDumps()
	{
		return true;
	}

	/**
	 * Enforces strict memory load/store ordering across the memory barrier call.
	 */
	CORE_API FORCENOINLINE static void MemoryBarrier();

	/**
	 * Set a handler to be called when there is a memory warning from the OS
	 *
	 * @param Handler	The handler to call
	 */
	static void SetMemoryWarningHandler(void (* Handler)(const FGenericMemoryWarningContext& Context))
	{
	}

	/**
	 * Determines if a warning handler has been set
	 */
	static bool HasMemoryWarningHandler()
	{
		return false;
	}

	FORCEINLINE static uint32 GetLastError()
	{
		return 0;
	}

	static void SetLastError(uint32 ErrorCode) {}

	static CORE_API void RaiseException( uint32 ExceptionCode );

public:

#if ENABLE_GENERIC_NAMED_EVENTS
	/** Indicates the start of a frame for named events, both generic and platform-specific. */
	static CORE_API void BeginNamedEventFrame();
	/** Add a named event that can be viewed in external tool, both generic and platform-specific. */
	static CORE_API void BeginNamedEvent(const struct FColor& Color, const TCHAR* Text);
	static CORE_API void BeginNamedEvent(const struct FColor& Color, const ANSICHAR* Text);
	/** Close a named event that can be viewed in external tool, both generic and platform-specific. */
	static CORE_API void EndNamedEvent();
	/** Set the value for a named custom stat that can be viewed in external tool, both generic and platform-specific. */
	static CORE_API void CustomNamedStat(const TCHAR* Text, float Value, const TCHAR* Graph, const TCHAR* Unit);
	static CORE_API void CustomNamedStat(const ANSICHAR* Text, float Value, const ANSICHAR* Graph, const ANSICHAR* Unit);
#else
	FORCEINLINE static void BeginNamedEventFrame()
	{
	}
	FORCEINLINE static void BeginNamedEvent(const struct FColor& Color, const TCHAR* Text)
	{
	}
	FORCEINLINE static void BeginNamedEvent(const struct FColor& Color, const ANSICHAR* Text)
	{
	}
	FORCEINLINE static void EndNamedEvent()
	{
	}
	FORCEINLINE static void CustomNamedStat(const TCHAR* Text, float Value, const TCHAR* Graph, const TCHAR* Unit)
	{
	}
	FORCEINLINE static void CustomNamedStat(const ANSICHAR* Text, float Value, const ANSICHAR* Graph, const ANSICHAR* Unit)
	{
	}
#endif


	/**
	 * Stats logging for named event 
	 */
	template<typename CharType>
	static void StatNamedEvent(const CharType* Text);
	static CORE_API void TickStatNamedEvents();
	static CORE_API void LogNameEventStatsInit();

	/**
	 * Profiler enter background events. These are for timing backgrounding. Some platforms have a timeout for this
	 */
	static void BeginEnterBackgroundEvent(const TCHAR* Text) {}
	static void EndEnterBackgroundEvent() {}


	/**
	 * Platform specific function for initializing storage of tagged memory buffers
	 */
	FORCEINLINE static void InitTaggedStorage(uint32 NumTags)
	{
	}

	/**
	 * Platform specific function for freeing storage of tagged memory buffers
	 */
	FORCEINLINE static void ShutdownTaggedStorage()
	{
	}

	/**
	 * Platform specific function for tagging a memory buffer with a label. Helps see memory access in profilers
	 */
	FORCEINLINE static void TagBuffer(const char* Label, uint32 Category, const void* Buffer, size_t BufferSize)
	{
	}

	/**
	 *	Set the value for the given section and key in the platform specific key->value store
	 *  Note: The key->value store is user-specific, but may be used to share data between different applications for the same user
	 *
	 *  @param	InStoreId			The name used to identify the store you want to use (eg, MyGame)
	 *	@param	InSectionName		The section that this key->value pair is placed within (can contain / separators, eg UserDetails/AccountInfo)
	 *	@param	InKeyValues			The mapping of key->value pairs to set
	 *	@return	bool				true if the value was set correctly, false if not
	 */
	static CORE_API bool SetStoredValues(const FString& InStoreId, const FString& InSectionName, const TMap<FString, FString>& InKeyValues);

	/**
	 *	Set the value for the given section and key in the platform specific key->value store
	 *  Note: The key->value store is user-specific, but may be used to share data between different applications for the same user
	 *
	 *  @param	InStoreId			The name used to identify the store you want to use (eg, MyGame)
	 *	@param	InSectionName		The section that this key->value pair is placed within (can contain / separators, eg UserDetails/AccountInfo)
	 *	@param	InKeyName			The name of the key to set the value for
	 *	@param	InValue				The value to set
	 *	@return	bool				true if the value was set correctly, false if not
	 */
	static CORE_API bool SetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, const FString& InValue);

	/**
	 *	Get the value for the given section and key from the platform specific key->value store
	 *  Note: The key->value store is user-specific, but may be used to share data between different applications for the same user
	 *
	 *  @param	InStoreId			The name used to identify the store you want to use (eg, MyGame)
	 *	@param	InSectionName		The section that this key->value pair is placed within (can contain / separators, eg UserDetails/AccountInfo)
	 *	@param	InKeyName			The name of the key to get the value for
	 *	@param	OutValue			The value found
	 *	@return	bool				true if the entry was found (and OutValue contains the result), false if not
	 */
	static CORE_API bool GetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, FString& OutValue);

	/**
	 *	Deletes value for the given section and key in the platform specific key->value store
	 *  Note: The key->value store is user-specific, but may be used to share data between different applications for the same user
	 *
	 *  @param	InStoreId			The name used to identify the store you want to use (eg, MyGame)
	 *	@param	InSectionName		The section that this key->value pair is placed within (can contain / separators, eg UserDetails/AccountInfo)
	 *	@param	InKeyName			The name of the key to set the value for
	 *	@return	bool				true if the value was deleted correctly, false if not found or couldn't delete
	 */
	static CORE_API bool DeleteStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName);

	/**
	 *	Deletes the given section and its contained values in the platform specific key->value store
	 *  Note: The key->value store is user-specific, but may be used to share data between different applications for the same user
	 *
	 *  @param	InStoreId			The name used to identify the store you want to use (eg, MyGame)
	 *	@param	InSectionName		The section to delete.
	 *	@return	bool				true if the section was deleted, false if not found or couldn't delete
	 */
	static CORE_API bool DeleteStoredSection(const FString& InStoreId, const FString& InSectionName);

	/** Sends a message to a remote tool, and debugger consoles */
	static CORE_API void LowLevelOutputDebugString(const TCHAR *Message);
	static CORE_API void VARARGS LowLevelOutputDebugStringf(const TCHAR *Format, ... );

	/** Whether LowLevelOutputDebugString[f] can receive structured output as lines of JSON. */
	static CORE_API bool IsLowLevelOutputDebugStringStructured();

	/** Sets the default output to UTF8 */
	static CORE_API void SetUTF8Output();

	/** Prints string to the default output */
	static CORE_API void LocalPrint( const TCHAR* Str );

	/** Whether LocalPrint can be called from any thread without overlapping */
	static bool IsLocalPrintThreadSafe() { return false;  }

	/**
	 * Whether the platform has a separate debug channel to stdout (eg. OutputDebugString on Windows). Used to suppress messages being output twice
	 * if both go to the same place.
	 */
	static CORE_API bool HasSeparateChannelForDebugOutput();

	/**
	 * Requests application exit.
	 *
	 * @param	Force	If true, perform immediate exit (dangerous because config code isn't flushed, etc).
	 *				  If false, request clean main-loop exit from the platform specific code.
	 * @param	CallSite a diagnostic message identifying the callsite.
	 */
	static CORE_API void RequestExit( bool Force, const TCHAR* CallSite = nullptr);

	/**
	 * Requests application exit with a specified return code. Name is different from RequestExit() so overloads of just one of functions are possible.
	 *
	 * @param	Force 	   If true, perform immediate exit (dangerous because config code isn't flushed, etc).
	 *					 If false, request clean main-loop exit from the platform specific code.
	 * @param   ReturnCode This value will be returned from the program (on the platforms where it's possible). Limited to 0-255 to conform with POSIX.
	 * @param	CallSite a diagnostic message identifying the callsite.
	 */
	static CORE_API void RequestExitWithStatus( bool Force, uint8 ReturnCode, const TCHAR* CallSite = nullptr);


	/**
	 * Requests application to restart
	 */
	static CORE_API bool RestartApplication();

    /**
	 * Requests application to restart appending the passed in CmdLine to the restart
	 */
	static CORE_API bool RestartApplicationWithCmdLine(const char* CmdLine);

	/**
	 * Returns the last system error code in string form.  NOTE: Only one return value is valid at a time!
	 *
	 * @param OutBuffer the buffer to be filled with the error message
	 * @param BufferLength the size of the buffer in character count
	 * @param Error the error code to convert to string form
	 */
	static CORE_API const TCHAR* GetSystemErrorMessage(TCHAR* OutBuffer, int32 BufferCount, int32 Error);

	/** Create a new globally unique identifier. **/
	static CORE_API void CreateGuid(struct FGuid& Result);

	/**
	 * Show a message box if possible, otherwise print a message and return the default
	 * @param MsgType What sort of options are provided
	 * @param Text Specific message
	 * @param Caption String indicating the title of the message box
	 * @return Very strange convention...not really EAppReturnType, see implementation
	 */
	static CORE_API EAppReturnType::Type MessageBoxExt( EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption );

	/**
	 * Handles Game Explorer, Firewall and FirstInstall commands, typically from the installer
	 * @returns false if the game cannot continue.
	 */
	static bool CommandLineCommands()
	{
		return 1;
	}

	/**
	 * Detects whether we're running in a 64-bit operating system.
	 *
	 * @return	true if we're running in a 64-bit operating system
	 */
	UE_DEPRECATED(5.8, "FPlatformMisc::Is64bitOperatingSystem is deprecated as UE only supports 64-bit.")
	FORCEINLINE static bool Is64bitOperatingSystem()
	{
		return true;
	}

	/**
	 * Checks structure of the path against platform formatting requirements
	 *
	 * return true if path is formatted validly
	 */
	static bool IsValidAbsolutePathFormat(const FString& Path)
	{
		return 1;
	}

	/**
	 * Platform-specific normalization of path
	 * E.g. on Linux/Unix platforms, replaces ~ with user home directory, so ~/.config becomes /home/joe/.config (or /Users/Joe/.config)
	 */
	static void NormalizePath(FString& InPath)
	{
	}

	/**
	 * Platform-specific normalization of path
	 * E.g. on Linux/Unix platforms, replaces ~ with user home directory, so ~/.config becomes /home/joe/.config (or /Users/Joe/.config)
	 */
	static void NormalizePath(FStringBuilderBase& InPath)
	{
	}

	/**
	 * @return platform specific path separator.
	 */
	static CORE_API const TCHAR* GetDefaultPathSeparator();

	/**
	 * Checks if platform wants to use a rendering thread on current device
	 *
	 * @return true if allowed, false if shouldn't use a separate rendering thread
	 */
	static CORE_API bool UseRenderThread();

	/**
	 * Checks if platform wants to allow an audio thread on current device (note: does not imply it will, only if okay given other criteria met)
	 *
	 * @return true if allowed, false if shouldn't use a separate audio thread
	 */
	static bool AllowAudioThread()
	{
		// allow if not overridden
		return true;
	}

	/**
	 * Checks if platform wants to allow the thread heartbeat hang detection
	 *
	 * @return true if allows, false if shouldn't allow thread heartbeat hang detection
	 */
	static CORE_API bool AllowThreadHeartBeat();

	/**
	 * return the number of hardware CPU cores
	 */
	static CORE_API int32 NumberOfCores();

	/**
	* @return a description of all the groups in the current system and their affinities
	*/
	static CORE_API const FProcessorGroupDesc& GetProcessorGroupDesc();

	/**
	 * return the number of logical CPU cores
	 */
	static CORE_API int32 NumberOfCoresIncludingHyperthreads();

	/**
	 * Return the number of worker threads we should spawn, based on number of cores
	 */
	static CORE_API int32 NumberOfWorkerThreadsToSpawn();

	/**
	 * Return the number of worker threads we should spawn to service IO, NOT based on number of cores
	 */
	static CORE_API int32 NumberOfIOWorkerThreadsToSpawn();

	/**
	 * Return the platform specific async IO system, or nullptr if the standard one should be used.
	 */
	static struct FAsyncIOSystemBase* GetPlatformSpecificAsyncIOSystem()
	{
		return nullptr;
	}

	/** Return the name of the platform features module. Can be nullptr if there are no extra features for this platform */
	static const TCHAR* GetPlatformFeaturesModuleName()
	{
		// by default, no module
		return nullptr;
	}

	/** Get the application root directory. */
	static CORE_API const TCHAR* RootDir();

	/** get additional directories which can be considered as root directories */
	static CORE_API TArray<FString> GetAdditionalRootDirectories();
	/** add an additional root directory */
	static CORE_API void AddAdditionalRootDirectory(const FString& RootDir);

	/** Get the engine directory */
	static CORE_API const TCHAR* EngineDir();

	/** Get the directory the application was launched from (useful for commandline utilities) */
	static CORE_API const TCHAR* LaunchDir();

	/** Function to store the current working directory for use with LaunchDir() */
	static CORE_API void CacheLaunchDir();

	/**
	 *	Return the project directory
	 */
	static CORE_API const TCHAR* ProjectDir();

	/**
	 *	Return the engine and project directory based from the executable.
	 *  This function would return different results than EngineDir and Project dir if for example -basedir is set on cmd line and/or pak files are used
	 */
	static CORE_API bool GetEngineAndProjectAbsoluteDirsFromExecutable(FString& OutProjectDir, FString& OutEngineDir);

	/**
	*	Return the CloudDir.  CloudDir can be per-user.
	*/
	static CORE_API FString CloudDir();

	/**
	*	Return true if the PersistentDownloadDir is available.
	*	On some platforms, a writable directory might not be available by default.
	*	Using this function allows handling that case early.
	*/
	static bool HasProjectPersistentDownloadDir()
	{
		return true;
	}

	/** Ensure that there enough space in the persistent download dir. Returns true if there is sufficient space.
	 *  If there is insufficient space and bAttemptToUseUI is true, the function will return false and the user may be prompted to clear space
	 */
	static CORE_API bool CheckPersistentDownloadStorageSpaceAvailable( uint64 BytesRequired, bool bAttemptToUseUI );


	/**
	 *	Return the GamePersistentDownloadDir.
	 *	On some platforms, returns the writable directory for downloaded data that persists across play sessions.
	 *	This dir is always per-game.
	 */
	static CORE_API const TCHAR* GamePersistentDownloadDir();

	/**
	 *	Return the temporary directory
	 */
	static CORE_API const TCHAR* GameTemporaryDownloadDir();

	static CORE_API const TCHAR* GeneratedConfigDir();

	static CORE_API const TCHAR* GetUBTPlatform();

	static CORE_API const TCHAR* GetUBTTarget();

	static CORE_API const TCHAR* GetUBTArchitecture();

	static CORE_API void SetUBTTargetName(const TCHAR* InTargetName);
	static CORE_API const TCHAR* GetUBTTargetName();

	/** 
	 * Determines the shader format for the platform
	 *
	 * @return	Returns the shader format to be used by that platform
	 */
	static CORE_API const TCHAR* GetNullRHIShaderFormat();

	/**
	 * Returns the platform specific chunk based install interface
	 *
	 * @return	Returns the platform specific chunk based install implementation
	 */
	static CORE_API IPlatformChunkInstall* GetPlatformChunkInstall();

	/**
	 * Returns the platform specific compression interface
	 *
	 * @return Returns the platform specific compression interface
	 */
	static CORE_API IPlatformCompression* GetPlatformCompression();

	/**
	 * Returns the platform specific interface for communication with processes running on the host pc.
	 * 
	 * @return Returns the platform specific host communication interface.
	 */
	static CORE_API IPlatformHostCommunication& GetPlatformHostCommunication();

	/**
	 * If bEnablePreferredInputAPIPreferences is enabled, then check if the given input device is
	 * "Preferred" by the user. If a null device name is given, then this function will always treat 
	 * that as a preferred API and return true.
	 * 
	 * @param DeviceName	The name of the input device API to check if it is preferred. This should come from
	 *						an IInputDeviceModule::GetPreferredDeviceAPIString call.
	 * @return True if the given input device API is preferred.
	 */
	static CORE_API bool IsPreferredInputDevice(const TCHAR* DeviceName);

	/**
	 * Sets the [GamepadAPIPreference]PreferredAPI config value in the GGameUserSettingsIni ini to the
	 * given value. This will then be used to determine if an input API is preferred on the next boot.
	 *
	 * NOTE: This will not take affect immediately. An engine reboot is required to determine which input APIs should be created.
	 * 
	 * @param DeviceName The full comma-separated list of preferred input APIs this application should use.
	 */
	static CORE_API void SetPreferredInputDevices(const TCHAR* DeviceName);

	/**
	 * Resets the users [GamepadAPIPreference]PreferredAPI config value
	 * and calls SetPreferredInputDevices with the project's default preferred
	 * input API list if available.
	 */
	static CORE_API void ResetPreferredInputDevicesToDefault();

	/**
	 * @return A comma-separated string value which represents a list of the preferred input APIs.
	 */
	static CORE_API FString GetCurrentPreferredInputDevices();

	/**
	 * Has the OS execute a command and path pair (such as launch a browser)
	 *
	 * @param ComandType OS hint as to the type of command
	 * @param Command the command to execute
	 * @param CommandLine the commands to pass to the executable
	 * @return whether the command was successful or not
	 */
	static bool OsExecute(const TCHAR* CommandType, const TCHAR* Command, const TCHAR* CommandLine = NULL)
	{
		return false;
	}

	/**
	 * @return true if this build is meant for release to retail
	 */
	static bool IsPackagedForDistribution()
	{
#if UE_BUILD_SHIPPING
		return true;
#else
		return false;
#endif
	}

	/**
	 * Generates the SHA256 signature of the given data.
	 *
	 *
	 * @param Data Pointer to the beginning of the data to hash
	 * @param Bytesize Size of the data to has, in bytes.
	 * @param OutSignature Output Structure to hold the computed signature.
	 *
	 * @return whether the hash was computed successfully
	 */
	static CORE_API bool GetSHA256Signature(const void* Data, uint32 ByteSize, FSHA256Signature& OutSignature);

	/**
	 * Get the default language (for localization) used by this platform.
	 * @note This is typically the same as GetDefaultLocale unless the platform distinguishes between the two.
	 * @note This should be returned in IETF language tag form:
	 *  - A two-letter ISO 639-1 language code (eg, "zh").
	 *  - An optional four-letter ISO 15924 script code (eg, "Hans").
	 *  - An optional two-letter ISO 3166-1 country code (eg, "CN").
	 */
	static CORE_API FString GetDefaultLanguage();

	/**
	 * Get the default locale (for internationalization) used by this platform.
	 * @note This should be returned in IETF language tag form:
	 *  - A two-letter ISO 639-1 language code (eg, "zh").
	 *  - An optional four-letter ISO 15924 script code (eg, "Hans").
	 *  - An optional two-letter ISO 3166-1 country code (eg, "CN").
	 */
	static CORE_API FString GetDefaultLocale();

	/**
	 * Get the timezone identifier for this platform, or an empty string if the default timezone calculation will work.
	 * @note This should return either an Olson timezone (eg, "America/Los_Angeles") or an offset from GMT/UTC (eg, "GMT-8:00").
	 */
	static CORE_API FString GetTimeZoneId();

	/**
	 *	Platform-specific Exec function
	 *
	 *  @param	InWorld		World context
	 *	@param	Cmd			The command to execute
	 *	@param	Out			The output device to utilize
	 *	@return	bool		true if command was processed, false if not
	 */
	static bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Out)
	{
		return false;
	}

	/** @return Get the name of the platform specific file manager (eg, Explorer on Windows, Finder on OS X) */
	static CORE_API FText GetFileManagerName();

	/** @return Whether filehandles can be opened on one thread and read/written on another thread */
	static bool SupportsMultithreadedFileHandles()
	{
		return true;
	}

#if !UE_BUILD_SHIPPING
	static void SetShouldPromptForRemoteDebugging(bool bInShouldPrompt)
	{
		bShouldPromptForRemoteDebugging = bInShouldPrompt;
	}

	static void SetShouldPromptForRemoteDebugOnEnsure(bool bInShouldPrompt)
	{
		bPromptForRemoteDebugOnEnsure = bInShouldPrompt;
	}
#endif	//#if !UE_BUILD_SHIPPING

	/**
	 * Allows disabling ensure()s without rebuilding the binary, by either a commandline switch or a hotfix.
	 *
	 * @return ensure is allowed
	 */
#if DO_ENSURE
	static CORE_API bool IsEnsureAllowed();
#else
	static bool IsEnsureAllowed() { return true; }
#endif // DO_ENSURE

	/**
	 * Updates hotfixable ensure settings from config and commandline (config takes priority).
	 */
#if DO_ENSURE
	static CORE_API void UpdateHotfixableEnsureSettings();
#else
	static void UpdateHotfixableEnsureSettings() {}
#endif // DO_ENSURE

	/**
	 * Ticks values that can be hotfixable in the config.
	 */
	static CORE_API void TickHotfixables();

	static void PromptForRemoteDebugging(bool bIsEnsure)
	{
	}

	FORCEINLINE static void Prefetch(const void* Ptr)
	{
#if PLATFORM_CPU_X86_FAMILY
		_mm_prefetch(static_cast<const char*>(Ptr), _MM_HINT_T0);
#elif PLATFORM_CPU_ARM_FAMILY
#	if PLATFORM_COMPILER_MSVC
		__prefetch(Ptr);
#	else
		__asm__ __volatile__("prfm pldl1keep, [%[ptr]]\n" ::[ptr] "r"(Ptr) : );
#	endif
#else
		// any other architecture must implement this function so this is not called, but we can't use any convenience functions 
		// like unimplemented() or checkf(), so just crash with a comment that if you get here, implement Prefetch and ALL Prefecth calls below
		// this is because the Prefetch calls below will end up calling _this one_ not the platform specific functions
		UE_FORCE_CRASH();
#endif
	}

	FORCEINLINE static void Prefetch(const void* Ptr, int32 Offset)
	{
		Prefetch(reinterpret_cast<const void*>(reinterpret_cast<UPTRINT>(Ptr) + Offset));
	}

	FORCEINLINE static void PrefetchBlock(const void* Ptr, int32 NumBytes)
	{
		// Use compile-time PLATFORM_CACHE_LINE_SIZE to avoid having to load a value before issuing prefetch instructions
		for (int32 Line = 0, NumLines = (NumBytes + PLATFORM_CACHE_LINE_SIZE - 1) / PLATFORM_CACHE_LINE_SIZE; Line < NumLines; ++Line)
		{
			Prefetch(Ptr, Line * PLATFORM_CACHE_LINE_SIZE);
		}
	}

	/**
	 * Gets the default profile name. Used if there is no device profile specified
	 *
	 * @return the default profile name.
	 */
	static CORE_API const TCHAR* GetDefaultDeviceProfileName();

	/**
	 * Gets the current battery level.
	 *
	 * @return the battery level between 0 and 100.
	 */
	FORCEINLINE static int GetBatteryLevel()
	{
		return -1;
	}

	FORCEINLINE static void SetBrightness(float bBright) { }
	FORCEINLINE static float GetBrightness() { return 1.0f; }
	FORCEINLINE static bool SupportsBrightness() { return false; }

	FORCEINLINE static bool IsInLowPowerMode() { return false;}

	/**
	 * Returns the current device temperature level.
	 * Level is a relative value that is platform dependent. Lower is cooler, higher is warmer.
	 * Level of -1 means Unimplemented.
	 */
	UE_DEPRECATED(5.8, "GetDeviceTemperatureLevel() has been superseded by GetDeviceTemperature() and/or GetDeviceThermalState() because it's behavior was poorly defined (on android it returned temperature, on ios it returned a thermal status enum value).")
	static CORE_API float GetDeviceTemperatureLevel();

	/**
	 * Returns the current device temperature in degrees Celcius.
	 * Keep in mind that no specific temperature probe position can be assumed.  It will vary across platforms and devices.
	 * Inspecting the platform implementation might yeild more information about this stat or more detailed platform specific stats.
	 * -1.0 is returned for a platform that does not support this measure at all.
	 * A platform that does support it, but is not currently providing data may return -1 or 0.  
	 * Theoretically -1 and 0 could be real temperatures... 
	 */
	static CORE_API float GetDeviceTemperature();

	/**
	 * Returns the current device thermal state, which is a cross platform qualitative representation of the device's heat status.
	 * Briefly EDeviceThermalState::None (0) is the ideal state, EDeviceThermalState::Severe+ (3) is where serious throttling will start.
	 * See EDeviceThermalState and device specific implemetations for details.
	 * -1.0 is returned for a platform that does not support this measure at all.
	 */
	static CORE_API EDeviceThermalState GetDeviceThermalState();

	/**
	 * Allows a game/program/etc to control the game directory in a special place (for instance, monolithic programs that don't have .uprojects)
	 */
	static CORE_API void SetOverrideProjectDir(const FString& InOverrideDir);

	UE_DEPRECATED(4.18, "FPaths::SetOverrideGameDir() has been superseded by FPaths::SetOverrideProjectDir().")
	static FORCEINLINE void SetOverrideGameDir(const FString& InOverrideDir) { return SetOverrideProjectDir(InOverrideDir); }

	/**
	 * Return an ordered list of target platforms this runtime can support (ie Android_DXT, Android
	 * would mean that it prefers Android_DXT, but can use Android as well)
	 */
	static CORE_API void GetValidTargetPlatforms(TArray<FString>& TargetPlatformNames);

	/*
	 * Converts from a local user index to a FPlatformUserId
	 *
	 * @param LocalUserIndex	A logical index where 0 should always be a valid user
	 * @returns					A platform-allocated opaque user id, or PLATFORMUSERID_NONE
	 */
	static CORE_API FPlatformUserId GetPlatformUserForUserIndex(int32 LocalUserIndex);

	/*
	 * Converts from a FPlatformUserId to local user index
	 *
	 * @param LocalUserIndex	A platform-allocated opaque user id
	 * @returns					A valid logical index, or INDEX_NONE
	 */
	static CORE_API int32 GetUserIndexForPlatformUser(FPlatformUserId PlatformUser);

	/**
	 * Returns whether the platform wants to use a touch screen for virtual joysticks.
	 */
	static bool GetUseVirtualJoysticks()
	{
		return PLATFORM_HAS_TOUCH_MAIN_SCREEN;
	}

	static bool SupportsTouchInput()
	{
		return PLATFORM_HAS_TOUCH_MAIN_SCREEN;
	}

	static bool SupportsForceTouchInput()
	{
		return false;
	}

	static bool ShouldDisplayTouchInterfaceOnFakingTouchEvents()
	{	// FSlateApplication::Get().IsFakingTouchEvents() will trigger to display the Touch Interface
		// on some platforms, we want to ignore that condition
		return true;
	}

	static bool DesktopTouchScreen()
	{
#if PLATFORM_DESKTOP
		return true;
#else
		return false;
#endif
	}

	static bool FullscreenSameAsWindowedFullscreen()
	{
		// On some platforms, Fullscreen and WindowedFullscreen behave the same.
		//	 e.g. On Linux, see FLinuxWindow::ReshapeWindow()/SetWindowMode()
		//		  Allowing Fullscreen window mode confuses higher level code (see UE-19996).
		return false;
	}

	/*
	 * Returns whether the volume buttons are handled by the system
	 */
	static bool GetVolumeButtonsHandledBySystem()
	{
		return true;
	}

	/*
	 * Set whether the volume buttons are handled by the system
	 */
	static void SetVolumeButtonsHandledBySystem(bool enabled)
	{}

	/** @return Memory representing a true type or open type font provided by the platform as a default font for unreal to consume; empty array if the default font failed to load. */
	static CORE_API TArray<uint8> GetSystemFontBytes();

	/**
	 * Returns whether WiFi connection is currently active
	 */
	static bool HasActiveWiFiConnection()
	{
		return false;
	}

	/**
	 * Returns the current status for the network connection
	 */
	static CORE_API ENetworkConnectionStatus GetNetworkConnectionStatus();

	/**
	 * Updates the current status for the network connection
	 */
	static CORE_API void SetNetworkConnectionStatus(ENetworkConnectionStatus NewNetworkConnectionStatus);

	/**
	 * Converts a network connection type to a network connection status.
	 * Maps None/AirplaneMode to Disabled, all known connection types to Connected.
	 */
	static ENetworkConnectionStatus NetworkConnectionTypeToStatus(ENetworkConnectionType ConnectionType)
	{
		switch (ConnectionType)
		{
		case ENetworkConnectionType::None:
		case ENetworkConnectionType::AirplaneMode:
			return ENetworkConnectionStatus::Disabled;
		case ENetworkConnectionType::Cell:
		case ENetworkConnectionType::WiFi:
		case ENetworkConnectionType::WiMAX:
		case ENetworkConnectionType::Bluetooth:
		case ENetworkConnectionType::Ethernet:
		case ENetworkConnectionType::Unknown:
		default:
			return ENetworkConnectionStatus::Connected;
		}
	}

	/**
	 * Returns whether WiFi connection is currently active
	 */
	static ENetworkConnectionType GetNetworkConnectionType()
	{
		return ENetworkConnectionType::Unknown;
	}
	
	static bool IsDataSavingNetworkConnection()
	{
		return false;
	}

	static void EnsureNetworkIfNecessary()	{}

	/**
	 * Returns whether the platform has variable hardware (configurable/upgradeable system).
	 */
	static bool HasVariableHardware()
	{
		// By default assume that platform hardware is variable.
		return true;
	}

	/**
	 * Returns whether the given platform feature is currently available (for instance, Metal is only available in IOS8 and with A7 devices)
	 */
	static bool HasPlatformFeature(const TCHAR* FeatureName)
	{
		return false;
	}

	/**
	 * Returns whether the device is considered a gaming handheld device. It's not a dynamic property, the status is cached at startup.
	 */
	static CORE_API bool IsDeviceGamingHandheld();

	/**
	 * Returns whether the platform is running on battery power or not.
	 */
	static CORE_API bool IsRunningOnBattery();

	/**
	 * Returns the current orientation of the device: will be either Portrait, LandscapeLeft, PortraitUpsideDown or LandscapeRight.
	 * 
	 * @return An EDeviceScreenOrientation value.
	 */
	static CORE_API EDeviceScreenOrientation GetDeviceOrientation();

	/**
	 * Returns the allowed orientation of the device. This is NOT the same as GetDeviceOrientation, which only returns Portrait, LandscapeLeft, 
	 * PortraitUpsideDown or LandscapeRight. The allowed orientation limits what orientation your device can have. So if you set the allowed orientation 
	 * to LandscapeLeft, GetDeviceOrientation will only ever return LandscapeLeft. But if you set the allowed orientation to LandscapeSensor, you are actually 
	 * restricting the allowed orientations to LandscapeLeft OR LandscapeRight (depending on the sensor), so GetDeviceOrientation might return LandscapeLeft OR LandscapeRight.
	 * 
	 * @return An EDeviceScreenOrientation value.
	 */
	static CORE_API EDeviceScreenOrientation GetAllowedDeviceOrientation();

	/**
	 * Change the allowed orientation of the device. 
	 */
	static CORE_API void SetAllowedDeviceOrientation(EDeviceScreenOrientation NewAllowedDeviceOrientation);
    
	/**
	 * Returns the device volume if the device is capable of returning that information.
	 *  -1 : Unknown
	 *   0 : Muted
	 * 100 : Full Volume
	 */
	static CORE_API int32 GetDeviceVolume();

	/**
	 * Get (or create) the unique ID used to identify this computer.
	 * This is sort of a misnomer, as there will actually be one per user account on the operating system.
	 * This is NOT based on a machine fingerprint.
	 */
	UE_DEPRECATED(4.14, "GetMachineId is deprecated. Use GetLoginId instead.")
	static CORE_API FGuid GetMachineId();

	/**
	 * Returns a unique string associated with the login account of the current machine.
	 * Implemented using persistent storage like the registry on window (using HKCU), so
	 * is susceptible to anything that could reset or revert that storage if the ID is created,
	 * which is generally during install or first run of the app.
	 *
	 * Note: This is NOT a user or machine fingerprint, as multiple logins on the same machine will
	 * not share the same ID, and it is not based on the hardware of the user. It is completely random and
	 * non-identifiable.
	 *
	 * @return a string containing the LoginID, or empty if not available.
	 */
	static CORE_API FString GetLoginId();

	/**
	 * Get the Epic account ID for the user who last used the Launcher.
	 *
	 * Note: This function is for Epic internal use only.  Do not use.
	 *
	 * @return an empty string if the account ID was not present or it failed to read it for any reason.
	 */
	static CORE_API FString GetEpicAccountId();

	/**
	 * Get Device Tag of the device running the session.
	 *
	 * Note: Gets from DeviceTag command line argument 
	 *
	 * @return empty string or the value from DeviceTag argument.
	 */
	static CORE_API FString GetDeviceTag();

	/**
	 * Gets a globally unique ID the represents a particular operating system install.
	 * @returns an opaque string representing the ID, or an empty string if the platform doesn't support one.
	 */
	static CORE_API FString GetOperatingSystemId();

	/**
	 * Gets the current mode of convertible laptops, i.e. Laptop or Tablet.
	 *
	 * @return The laptop mode, or Unknown if not known, or NotSupported if not a convertible laptop.
	 */
	static CORE_API EConvertibleLaptopMode GetConvertibleLaptopMode();

	/**
	 * Get a string description of the mode the engine was running in.
	 */
	static CORE_API const TCHAR* GetEngineMode();

	/**
	 * Returns an array of the user's preferred languages in order of preference
	 * @return An array of language IDs ordered from most preferred to least
	 */
	static CORE_API TArray<FString> GetPreferredLanguages();

	/**
	 * Returns the currency code associated with the device's locale
	 * @return the currency code associated with the device's locale
 	*/
	static CORE_API FString GetLocalCurrencyCode();

	/**
	 * Returns the currency symbol associated with the device's locale
	 * @return the currency symbol associated with the device's locale
	 */
	static CORE_API FString GetLocalCurrencySymbol();

	/**
	 * Requests permission to send remote notifications to the user's device.
	 */
	static CORE_API void RegisterForRemoteNotifications();

	/**
	 * Returns whether or not the device has been registered to receive remote notifications.
	 */
	static CORE_API bool IsRegisteredForRemoteNotifications();

	/**
	* Requests unregistering from receiving remote notifications on the user's device.
	*/
	static CORE_API void UnregisterForRemoteNotifications();

	/**
	* Register an audio focus change listener
	*/
	static CORE_API FDelegateHandle RegisterAudioFocusChangedEventListener(const FOnAudioFocusChangedDelegate Callback);

	/**
	* Unregister an audio focus change listener
	*/
	static CORE_API void UnregisterAudioFocusChangedEventListener(const FDelegateHandle DelegateHandle);

	/**
	* Returns activation protocol that's waiting to be broadcast
	*/
	static CORE_API FString GetPendingActivationProtocol();

	/**
	 * Allows platform at runtime to disable unsupported plugins
	 *  @param	PluginName	Name of enabled plugin to consider
	 *	@return	bool		true if plugin should be disabled
	 */
	static bool ShouldDisablePluginAtRuntime(const FString& PluginName)
	{
		return false;
	}

	/**
	 * For mobile devices, this will crank up a haptic engine for the specified type to be played later with TriggerMobileHaptics
	 * If this is called again before Release, it will switch to this type
	 */
	static void PrepareMobileHaptics(EMobileHapticsType Type)
	{
	}

	/**
	 * For mobile devices, this will kick the haptic type that was set in PrepareMobileHaptics. It can be called multiple times
	 * with only a single call to Prepare
	 */
	static void TriggerMobileHaptics()
	{
	}

	/**
	 * For mobile devices, this will shutdown the haptics, allowing system to put it to reset as needed
	 */
	static void ReleaseMobileHaptics()
	{
	}

	/**
	 * Perform a mobile-style sharing of a URL. Will use native UI to display sharing target
	 */
	static void ShareURL(const FString& URL, const FText& Description, int32 LocationHintX, int32 LocationHintY)
	{
	}

	static bool SupportsDeviceCheckToken()
	{
		return false;
	}

	static CORE_API bool RequestDeviceCheckToken(TFunction<void(const TArray<uint8>&)> QuerySucceededFunc, TFunction<void(const FString&, const FString&)> QueryFailedFunc);

	/**
	 * Loads a text file relative to the package root on platforms that distribute apps in package formats.
	 * For other platforms, the path is relative to the root directory.
	 */
	static CORE_API FString LoadTextFileFromPlatformPackage(const FString& RelativePath);

	static CORE_API bool FileExistsInPlatformPackage(const FString& RelativePath);

	/**
	 * Frees any memory retained by FGenericPlatformMisc.
	 */
	static CORE_API void TearDown();

	static CORE_API void ParseChunkIdPakchunkIndexMapping(TArray<FString> ChunkIndexRedirects, TMap<int32, int32>& OutMapping);

	static void PumpMessagesOutsideMainLoop()
	{
	}

	static void PumpMessagesForSlowTask()
	{
	}
	/**
	 *  Pumps app messages only if there are essential keep-alive messages pending. This function called from time-sensitive 
	 *  parts of the code and should take minimal time if there are no essential messages waiting
	 */
	static void PumpEssentialAppMessages()
	{
	}

	static void HidePlatformStartupScreen()
	{

	}

	FORCEINLINE static bool UseHDRByDefault()
	{
		return false;
	}

	FORCEINLINE static void ChooseHDRDeviceAndColorGamut(uint32 DeviceId, uint32 DisplayNitLevel, EDisplayOutputFormat& OutputDevice, EDisplayColorGamut& ColorGamut)
	{
	}

	FORCEINLINE static int32 GetChunkIDFromPakchunkIndex(int32 PakchunkIndex)
	{
		return PakchunkIndex;
	}

	static CORE_API FStringView GetPakFilenamePrefix();

	static CORE_API int32 GetPakchunkIndexFromPakFile(FStringView InFilename);

	FORCEINLINE static bool Expand16BitIndicesTo32BitOnLoad()
	{
		return false;
	}

	/**
	 * Returns any platform-specific key-value data that needs to be sent to the network file server
	 */
	FORCEINLINE static void GetNetworkFileCustomData(TMap<FString,FString>& OutCustomPlatformData)
	{
	}

	FORCEINLINE static bool SupportsBackbufferSampling()
	{
		return true;
	}

	/**
	 * retrieves the current maximum refresh rate supported by the platform
	 */
	static inline int32 GetMaxRefreshRate()
	{
		return 60;
	}

	/**
	 * retrieves the maximum refresh rate supported by the platform
	 */
	static inline int32 GetMaxSupportedRefreshRate()
	{
		return GetMaxRefreshRate();
	}

	/**
	 * Returns the platform's maximum allowed value for rhi.SyncInterval
	 */
	static inline int32 GetMaxSyncInterval()
	{
		// Generic platform has no limit.
		return MAX_int32;
	}
	
	/*
	* Returns the value of config rule variable or null if not found.
	*/
	static CORE_API const FString* GetConfigRulesVariable(const FString& Key);

	/*
	* Returns the utf8 view of the config rule variable or empty if not found.
	*/
	static CORE_API const FUtf8StringView GetConfigRulesVariableAsUtf8(const FUtf8StringView Key);

	/*
	* Returns the map containing all config rule variables.
	*/
	static CORE_API const class TMap<FString, FString>& GetConfigRuleVars();
 
	/**
	 * Returns true if PGO is currently enabled
	 */
	static CORE_API bool IsPGOEnabled();

	/**
	 * Returns true if this binary has been instrumented for collection of data needed for profile-guided optimization (PGO)
	 * This value is not expected to change during the run-time
	 */
	static CORE_API bool IsPGICapableBinary();

	/**
	 * On platforms that support run-time PGI on/off, starts a new collection with a given file name.
	 *
	 * @param AbsoluteFileName File to put the collection into
	 * @return true if everything was successful
	 */
	static bool StartNewPGOCollection(const FString& AbsoluteFileName) { return false; }

	/**
	 * Returns true if PGI (profile-guided instrumentation, needed for PGO) is currently ongoing.
	 * Note that on some platforms can start and stop profile data collection.
	 */
	static CORE_API bool IsPGIActive();

	static CORE_API const FString VersionCheckPlatformName();

	/**
	 * On platforms that support run-time PGI on/off, stops the current collection and writes out the file (or completes its write).
	 * The filename must have been given to the StartNewPGICollection() at the start of collection (it is needed in advance since the
	 * collection process may stream to it).
	 *
	 * @return true if everything was successful
	 */
	static bool StopPGOCollectionAndCloseFile() { return false; }

	static CORE_API void ShowConsoleWindow();

	/*
	 * For mobile devices, this function will return true if we wish to propagate the alpha to the backbuffer
	 */
	static CORE_API int GetMobilePropagateAlphaSetting();

	/**
	*	Return if the game is running on cloud server
	*/
	static bool IsRunningInCloud()
	{
		return false;
	}

	/**
	*	Return true if the game is being streamed to a separate client device
	*/
	static bool WasLaunchedStreaming()
	{
		return false;
	}

	static void DisableScreenTimeout()
	{
	}

	static void EnableScreenTimeout()
	{
	}

	/*
	 * Unix only function, but put a stub so we can call this for higher level code when we want to setup the Syscall Filters
	 */
	static bool SetupSyscallFilters()
	{
		return false;
	}

	static const float GetVirtualKeyboardInputHeight()
	{
		return 0.0f;
	}
	
	/*
	* Utility to sanitize names provided to FPlatformSystemWideMutex. 
	* Sanitization is done in-place and will not change the string length.
	*/
	static void SanitizeSystemWideMutexName(FString& InName);

#if !UE_BUILD_SHIPPING
	/**
	 * Returns any platform specific warning messages we want printed on screen
	 */
	static bool GetPlatformScreenWarnings(TArray<FText>& PlatformScreenWarnings)
	{
		return false;
	}

protected:
	/** Whether the user should be prompted to allow for a remote debugger to be attached */
	static CORE_API bool bShouldPromptForRemoteDebugging;
	/** Whether the user should be prompted to allow for a remote debugger to be attached on an ensure */
	static CORE_API bool bPromptForRemoteDebugOnEnsure;
#endif	//#if !UE_BUILD_SHIPPING

	static CORE_API EDeviceScreenOrientation AllowedDeviceOrientation;

	static CORE_API ENetworkConnectionStatus CurrentNetworkConnectionStatus;

protected:
	/**
	 * Parse CoreLimit from CommandLine or from the SetCoreLimit value. The real number of physical cores and logical
	 * cores must be provided so that CommandLine arguments can be translated.
	 * 
	 * @param bOutFullyInitialized True if commandline was available. If false, values are set to defaults.
	 * @param OutPhysicalCoreLimit The maximum number of physical cores to report in NumberOfCores.
	 *        Non-negative. 0 indicates unlimited.
	 * @param OutLogicalCoreLimit The maximum number of logical cores to report in NumberOfCoresIncludingHyperthreads.
	 *        Non-negative. 0 indicates unlimited.
	 * @param bOutSetPhysicalCountToLogicalCount Whether NumberOfCores should be overridden to return
	 *        NumberOfCoresIncludingHyperthreads.
	 */
	static CORE_API void GetConfiguredCoreLimits(int32 PlatformNumPhysicalCores, int32 PlatformNumLogicalCores,
		bool& bOutFullyInitialized, int32& OutPhysicalCoreLimit, int32& OutLogicalCoreLimit,
		bool& bOutSetPhysicalCountToLogicalCount);

private:
	struct FStaticData;
};


