// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Timespan.h"

namespace UE
{

/**
* TGenericPlatformSharedMutex - Read/Write Mutex
*	- Provides non-recursive Read/Write (or shared-exclusive) access.
*	- As a fallback default for non implemented platforms, using a single mutex to provide complete single mutual exclusion - no separate Read/Write access.
*/
template <class MutexType>
class TGenericPlatformSharedMutex
{
public:
	TGenericPlatformSharedMutex(const TGenericPlatformSharedMutex&) = delete;
	TGenericPlatformSharedMutex& operator=(const TGenericPlatformSharedMutex&) = delete;

	TGenericPlatformSharedMutex() = default;
	~TGenericPlatformSharedMutex() = default;

	UE_FORCEINLINE_HINT bool TryLock()
	{
		return Mutex.TryLock();
	}

	UE_FORCEINLINE_HINT void Lock()
	{
		Mutex.Lock();
	}

	UE_FORCEINLINE_HINT void Unlock()
	{
		Mutex.Unlock();
	}

	UE_FORCEINLINE_HINT bool TryLockShared()
	{
		return Mutex.TryLock();
	}

	UE_FORCEINLINE_HINT void LockShared()
	{
		Mutex.Lock();
	}

	UE_FORCEINLINE_HINT void UnlockShared()
	{
		Mutex.Unlock();
	}

private:
	MutexType Mutex;
};

void SanitizeSystemWideMutexName(FString& InName);

/** Platforms that don't need a working FPlatformSystemWideMutex can alias this one. */
class FPlatformSystemWideMutexNotImplemented
{
public:
	FPlatformSystemWideMutexNotImplemented(const FPlatformSystemWideMutexNotImplemented&) = delete;
	FPlatformSystemWideMutexNotImplemented& operator=(const FPlatformSystemWideMutexNotImplemented&) = delete;

	/** Construct a named, system-wide mutex and attempt to get access/ownership of it. */
	CORE_API explicit FPlatformSystemWideMutexNotImplemented(const FString& Name, FTimespan Timeout = FTimespan::Zero());

	/** Destructor releases system-wide mutex if it is currently owned. */
	~FPlatformSystemWideMutexNotImplemented() = default;

	/**
	 * Does the calling thread have ownership of the system-wide mutex?
	 *
	 * @return True if obtained. WARNING: Returns true for an owned but previously abandoned locks so shared resources can be in undetermined states. You must handle shared data robustly.
	 */
	bool IsValid() const
	{
		return false;
	}

	/** Releases system-wide mutex if it is currently owned. */
	void Release()
	{
	}
};

} // UE
