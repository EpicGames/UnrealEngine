// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "CoreTypes.h"

#if PLATFORM_SUPPORTS_NAMED_PIPES

/**
 * Wrapper for platform named pipe communication.
 */
class FGenericPlatformNamedPipe
{
public:

	/** Default constructor. */
	CORE_API FGenericPlatformNamedPipe();

	/** Virtual destructor. */
	CORE_API virtual ~FGenericPlatformNamedPipe();

	FGenericPlatformNamedPipe(const FGenericPlatformNamedPipe&) = delete;
	FGenericPlatformNamedPipe& operator=(const FGenericPlatformNamedPipe&) = delete;

public:

	/** Create a named pipe as a server or client, using overlapped IO if bAsync=1. */
	virtual bool Create(const FString& PipeName, bool bServer, bool bAsync) { return false; }

	/** Get the pipe name used on Create(). */
	virtual const FString& GetName() const
	{
		return *NamePtr;
	}

	/** Destroy the pipe. */
	virtual bool Destroy() { return false; }

	/** Open a connection from a client. */
	virtual bool OpenConnection() { return false; }

	/** Block if there's an IO operation in progress until it's done or errors out. */
	virtual bool BlockForAsyncIO() { return false; }

	/** Let the user know if the pipe is ready to send or receive data. */
	virtual bool IsReadyForRW() const { return false; }

	/** Update status of async state of the current pipe. */
	virtual bool UpdateAsyncStatus() { return false; }

	/** Write a buffer out. */
	virtual bool WriteBytes(int32 NumBytes, const void* Data) { return false; }

	/** Write a single int32. */
	inline bool WriteInt32(int32 In)
	{
		return WriteBytes(sizeof(int32), &In);
	}

	/* Read a buffer in. */
	virtual bool ReadBytes(int32 NumBytes, void* OutData) { return false; }

	/** Read a single int32. */
	inline bool ReadInt32(int32& Out)
	{
		return ReadBytes(sizeof(int32), &Out);
	}

	/** Returns true if the pipe has been created and hasn't been destroyed. */
	virtual bool IsCreated() const { return false; }

	/** Return true if the pipe has had any communication error. */
	virtual bool HasFailed() const { return true; }

protected:

	FString* NamePtr;
};


#endif
