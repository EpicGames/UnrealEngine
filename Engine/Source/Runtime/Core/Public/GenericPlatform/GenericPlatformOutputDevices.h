// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatform.h"
#include "HAL/CriticalSection.h"
#include "Templates/UniquePtr.h"

class FFeedbackContext;
class FOutputDevice;
class FOutputDeviceConsole;
class FOutputDeviceError;

/**
 * Generic implementation for most platforms
 */
struct FGenericPlatformOutputDevices
{
	/** Add output devices which can vary depending on platform, configuration, command line parameters. */
	CORE_API static void				SetupOutputDevices();

	/**
	 * Returns the absolute log filename generated from the project properties and/or command line parameters.
	 * The returned value may change during the execution, may not exist yet or may be locked by another process.
	 * It depends if the function is called before the log file was successfully opened or after. The log file is
	 * open lazily.
	 */
	CORE_API static FString				GetAbsoluteLogFilename();
	CORE_API static FOutputDevice*		GetLog();
	CORE_API static void				GetPerChannelFileOverrides(TArray<FOutputDevice*>& OutputDevices);
	static FOutputDevice*				GetEventLog()
	{
		return nullptr; // normally only used for dedicated servers
	}

	CORE_API static FOutputDeviceError*	GetError();
	CORE_API static FFeedbackContext*   GetFeedbackContext();

protected:
	static void ResetCachedAbsoluteFilename();

private:
	static constexpr SIZE_T AbsoluteFileNameMaxLength = 1024;
	static TCHAR CachedAbsoluteFilename[AbsoluteFileNameMaxLength];

	static void OnLogFileOpened(const TCHAR* Pathname);
	static FCriticalSection LogFilenameLock;
};
