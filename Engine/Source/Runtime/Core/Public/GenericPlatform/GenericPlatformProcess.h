// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformAffinity.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"
#include "Templates/Tuple.h"
#include "Templates/SharedPointer.h"

class FEvent;

////////////////////////////////////////////////////////////////////////////////
#if PLATFORM_CPU_X86_FAMILY
#include <immintrin.h>
#endif

////////////////////////////////////////////////////////////////////////////////
#if PLATFORM_APPLE
#include <mach/mach_time.h>
#endif

////////////////////////////////////////////////////////////////////////////////
#if !defined(__clang__)
#	include <intrin.h>
#	if defined(_M_ARM)
#		include <armintr.h>
#	elif PLATFORM_WINDOWS && PLATFORM_CPU_ARM_FAMILY
#		include <arm64intr.h>
#	endif
#endif

// Assume ThreadRipper is the max number of cores we can have
#if PLATFORM_DESKTOP
#define MAX_NUM_PROCESSORS 128
#else
#define MAX_NUM_PROCESSORS 16
#endif


class Error;
struct FProcHandle;
template <typename FuncType> class TFunctionRef;

namespace EProcessResource
{
	enum Type
	{
		/** 
		 * Limits address space - basically limits the largest address the process can get. Affects mmap() (won't be able to map files larger than that) among others.
		 * May also limit automatic stack expansion, depending on platform (e.g. Linux)
		 */
		VirtualMemory
	};
}


/** Not all platforms have different opening semantics, but Windows allows you to specify a specific verb when opening a file. */
namespace ELaunchVerb
{
	enum Type
	{
		/** Launch the application associated with opening file to 'view' */
		Open,

		/** Launch the application associated with opening file to 'edit' */
		Edit,
	};
}

/** Forward declaration for ENamedThreads */
namespace ENamedThreads
{
	enum Type : int32;
}

namespace UE::Core
{
	class FURLRequestFilter;
}

/** Generic implementation for the process handle. */
template< typename T, T InvalidHandleValue >
struct TProcHandle
{
	typedef T HandleType;
public:

	/** Default constructor. */
	UE_FORCEINLINE_HINT constexpr TProcHandle()
		: Handle( InvalidHandleValue )
	{ }

	/** Initialization constructor. */
	UE_FORCEINLINE_HINT explicit TProcHandle( T Other )
		: Handle( Other )
	{ }

	/** Accessors. */
	UE_FORCEINLINE_HINT T Get() const
	{
		return Handle;
	}

	UE_FORCEINLINE_HINT void Reset()
	{
		Handle = InvalidHandleValue;
	}

	UE_FORCEINLINE_HINT bool IsValid() const
	{
		return Handle != InvalidHandleValue;
	}

protected:
	/** Platform specific handle. */
	T Handle;
};


struct FProcHandle;

/** Generic implementation of the per-process memory stats. */
struct FPlatformProcessMemoryStats
{
	/** The amount of physical memory used by the process, in bytes. */
	uint64 UsedPhysical;

	/** The peak amount of physical memory used by the process, in bytes. */
	uint64 PeakUsedPhysical;

	/** Total amount of virtual memory used by the process, in bytes. */
	uint64 UsedVirtual;

	/** The peak amount of virtual memory used by the process, in bytes. */
	uint64 PeakUsedVirtual;
};

// Only used on Unix-like (Linux, Unix, Android) platforms because you have to scrape /proc/stat
namespace UE::Profiling
{
	struct FCPUStatTime{
		uint64_t			TotalTime;
		uint64_t			UserTime;
		uint64_t			NiceTime;
		uint64_t			SystemTime;
		uint64_t			SoftIRQTime;
		uint64_t			IRQTime;
		uint64_t			IdleTime;
		uint64_t			IOWaitTime;
	};
	
	struct FCPUState
	{
		constexpr static int32		MaxSupportedCores = MAX_NUM_PROCESSORS; 
		int32						CoreCount;
		int32						ActivatedCoreCount;
		ANSICHAR					Name[6];
		FCPUStatTime				CurrentUsage[MaxSupportedCores]; 
		FCPUStatTime				PreviousUsage[MaxSupportedCores];
		int32						Status[MaxSupportedCores];
		double						Utilization[MaxSupportedCores];
		double						IdleTime[MaxSupportedCores];
		double						AverageUtilization;
		double						AverageIdleTime;
	};
}

namespace UE::HAL
{
	struct FReadHandle
	{
		void* Value = nullptr;
		
		operator void*() const
		{
			return Value;
		}
	};
	
	struct FWriteHandle
	{
		void* Value = nullptr;
		
		operator void*() const
		{
			return Value;
		}
	};
	
	struct FProcessStartInfo
	{
		const TCHAR* Uri;
		const TCHAR* Arguments = TEXT("");
		const TCHAR* WorkingDirectory = nullptr;
		int32 PriorityModifier = 0;
		bool bDetached = false;
		bool bHidden = false;
		bool bInheritHandles = false;
		bool bBreakawayFromJob = false;
		FReadHandle StdIn;
		FWriteHandle StdOut;
		FWriteHandle StdErr = StdOut;
	};
	
	enum struct EProcessId: uint32 {};
}


/**
  * A helper class that creates and checks for existence of a sentinel file that can be tied to process lifetime. 
  * NOTE: The generic version will not handle the case where a process just dies without crashing - the file
  * will not be deleted. The three desktop platforms have subclasses that will make sure that 
  * DoesSentinelFileExist will return false if the process that created it is not running. DO NOT assume 
  * FileExists() type calls would be accurate (and also there's not a public method to retrieve the path to 
  * discourage such a check anyway).
 */
class IProcessSentinel
{
public:
	/** Query for existence of a sentinel file (usually from another process than created it) */	
	static bool CORE_API DoesSentinelFileExist(const TCHAR* Name, FString* OptionalOutContents=nullptr);
	
	/**
	 * Create the sentinel file, if possible 
	 * @return false if the file could not be created (because another process has it locked)
	 */
	virtual bool CreateSentinelFile(const TCHAR* Name, const FString& Contents) = 0;

protected:	
	/** Helper to create the path for the sentinel file. Defaults to ProcessIntermediate() dir */
	virtual FString CORE_API MakeFilename(const TCHAR* Name);
	
	/** 
	 * The actual implementation to determine if the sentinel file is exists and is valid (due to a process dying,
	 * the file may actually exist on disk, but isn't a valid file anymore, so a simple FileExists may not be good enough
	 */
	virtual bool DoesSentinelFileExistImpl(const TCHAR* Name, FString* OptionalOutContents=nullptr) = 0;
};

/**
* Generic implementation for most platforms, these tend to be unused and unimplemented
**/
struct FGenericPlatformProcess
{
	/**
	 * Generic representation of a interprocess semaphore
	 */
	struct FSemaphore
	{
		/** Returns the name of the object */
		const TCHAR* GetName() const
		{
			return Name;
		}

		/** Acquires an exclusive access (also known as Wait()) */
		virtual void Lock() = 0;

		/** 
		 * Tries to acquire and exclusive access for a specified amount of nanoseconds (also known as TryWait()).
		 *
		 * @param Nanoseconds (10^-9 seconds) to wait for, 
		 * @return false if was not able to lock within given time
		 */
		virtual bool TryLock(uint64 NanosecondsToWait) = 0;

		/** Relinquishes an exclusive access (also known as Release())*/
		virtual void Unlock() = 0;

		/** 
		 * Creates and initializes a new instance with the specified name.
		 *
		 * @param InName name of the semaphore (all processes should use the same)
		 */
		FSemaphore(const FString& InName);

		/**
		 * Creates and initializes a new instance with the specified name.
		 *
		 * @param InName name of the semaphore (all processes should use the same)
		 */
		FSemaphore(const TCHAR* InName);

		/** Virtual destructor. */
		virtual ~FSemaphore() = default;

	protected:

		enum Limits
		{
			MaxSemaphoreName = 128
		};

		/** Name of the region */
		TCHAR			Name[MaxSemaphoreName];
	};

	/** Load a DLL. **/
	static CORE_API void* GetDllHandle( const TCHAR* Filename );

	/** Free a DLL. **/
	static CORE_API void FreeDllHandle( void* DllHandle );

	/** Lookup the address of a DLL function. **/
	static CORE_API void* GetDllExport( void* DllHandle, const TCHAR* ProcName );

	/** Adds a directory to search in when resolving implicitly loaded or filename-only DLLs. **/
	UE_FORCEINLINE_HINT static void AddDllDirectory(const TCHAR* Directory)
	{

	}

	/** Set a directory to look for DLL files. NEEDS to have a Pop call when complete */
	UE_FORCEINLINE_HINT static void PushDllDirectory(const TCHAR* Directory)
	{
	
	}

	/** Unsets a directory to look for DLL files. The same directory must be passed in as the Push call to validate */
	UE_FORCEINLINE_HINT static void PopDllDirectory(const TCHAR* Directory)
	{

	}

	/** Get the list of registered directories to search in when resolving implicitly loaded or filename-only DLLs. **/
	UE_FORCEINLINE_HINT static void GetDllDirectories(TArray<FString>& OutDllDirectories)
	{

	}

	/**
	 * Retrieves the ProcessId of this process.
	 *
	 * @return the ProcessId of this process.
	 */
	static CORE_API uint32 GetCurrentProcessId();

	/**
	 * Retrieves the current hardware CPU core
	 *
	 * @return the current hardware core.
	 */
	static CORE_API uint32 GetCurrentCoreNumber();
	
	/**
	 * Retrieves the current CPU Utilization for the current process in terms of percentage of all capacity
	 *
	 * @return the current hardware core.
	 */
	static bool GetPerFrameProcessorUsage(uint32 ProcessId, float& ProcessUsageFraction, float& IdleUsageFraction) {return false;}

	/**	 
	 * Change the thread processor affinity
	 *
	 * @param AffinityMask A bitfield indicating what processors the thread is allowed to run on.
	 */
	static CORE_API void SetThreadAffinityMask( uint64 AffinityMask );

	/**
	 * Change the thread processor priority
	 *
	 * @param NewPriority an EThreadPriority indicating what priority the thread is to run at.
	 */
	static CORE_API void SetThreadPriority( EThreadPriority NewPriority );

	/**
	 * Helper function to set thread name of the current thread.
	 * @param ThreadName   Name to set
	 */
	static void SetThreadName( const TCHAR* ThreadName ) { }

	/** Get the active stack size for the currently running thread **/
	static CORE_API uint32 GetStackSize();

	/** Get Default Thread Stack Size in bytes, 0 means to use the platform OS default size **/
	static uint32 GetDefaultThreadStackSize()
	{
		return 0;
	}

	/** Get Default Pool Thread Stack Size in bytes **/
	static uint32 GetDefaultPoolThreadStackSize()
	{
		return 128 * 1024;
	}

	/** Output information about the currently active thread **/
	static void DumpThreadInfo( const TCHAR* MarkerName ) { }

	/** Allow the platform to do anything it needs for game thread */
	static void SetupGameThread() { }

	/** Allow the platform to do anything it needs for render thread */
	static void SetupRenderThread() { }

	/** Allow the platform to do anything it needs for audio thread */
	static void SetupAudioThread() { }

	/** Allow the platform to tear down the audio thread */
	static void TeardownAudioThread() { }
	
	/** Content saved to the game or engine directories should be rerouted to user directories instead **/
	static CORE_API bool ShouldSaveToUserDir();

	/** Get startup directory.  NOTE: Only one return value is valid at a time! **/
	static CORE_API const TCHAR* BaseDir();

	/** Get user directory.  NOTE: Only one return value is valid at a time! **/
	static CORE_API const TCHAR* UserDir();

	/** Get the user settings directory.  NOTE: Only one return value is valid at a time! **/
	static CORE_API const TCHAR *UserSettingsDir();

	/** Get the user temporary directory.  NOTE: Only one return value is valid at a time! **/
	static CORE_API const TCHAR *UserTempDir();

	/** Get the user home directory.  NOTE: Only one return value is valid at a time! **/
	static CORE_API const TCHAR *UserHomeDir();

	struct ApplicationSettingsContext
	{
		enum class Context : int8_t
		{
			LocalUser,
			RoamingUser,
			ApplicationSpecific
		};
		Context Location;
		bool bIsEpic;
	};

	/** Get application settings directory.  NOTE: Only one return value is valid at a time! **/
	static CORE_API const TCHAR* ApplicationSettingsDir();
	
	/** 
	 * Get application settings directory for a given context.
	 * 
	 * @param Settings 	The context in which the application settings should be stored for.
	 * 
	 * @return 			A string to the appropriate directory.
	 */
	static CORE_API FString GetApplicationSettingsDir(const ApplicationSettingsContext& Settings);

	/** Get computer name.  NOTE: Only one return value is valid at a time! **/
	static CORE_API const TCHAR* ComputerName();

	/** Get user name.  NOTE: Only one return value is valid at a time! **/
	static CORE_API const TCHAR* UserName(bool bOnlyAlphaNumeric = true);
	static CORE_API const TCHAR* ShaderDir();
	static CORE_API void SetShaderDir(const TCHAR*Where);
	static CORE_API void SetCurrentWorkingDirectoryToBaseDir();

	/** Get the current working directory (only really makes sense on desktop platforms) */
	static CORE_API FString GetCurrentWorkingDirectory();

	/**
	 * Sets the process limits.
	 *
	 * @param Resource one of process resources.
	 * @param Limit the maximum amount of the resource (for some OS, this means both hard and soft limits).
	 * @return true on success, false otherwise.
	 */
	static bool SetProcessLimits(EProcessResource::Type Resource, uint64 Limit)
	{
		return true;	// return fake success by default, that way the game won't early quit on platforms that don't implement this
	}

	/**
	 * Get the shader working directory.
	 *
	 * @return The path to the directory.
	 */
	static CORE_API const FString ShaderWorkingDir();

	/**	Clean the shader working directory. */
	static CORE_API void CleanShaderWorkingDir();

	/**
	 * Return the path to the currently running executable
	 *
	 * @return 	Path of the currently running executable
	 */
	static CORE_API const TCHAR* ExecutablePath();

	/**
	 * Return the name of the currently running executable
	 *
	 * @param	bRemoveExtension	true to remove the extension of the executable name, false to leave it intact
	 * @return 	Name of the currently running executable
	 */
	static CORE_API const TCHAR* ExecutableName(bool bRemoveExtension = true);

	/**
	 * Generates the path to the specified application or game.
	 *
	 * The application must reside in the Engine's binaries directory. The returned path is relative to this
	 * executable's directory.For example, calling this method with "UE4" and EBuildConfiguration::Debug
	 * on Windows 64-bit will generate the path "../Win64/UnrealEditor-Win64-Debug.exe"
	 *
	 * @param AppName The name of the application or game.
	 * @param BuildConfiguration The build configuration of the game.
	 * @return The generated application path.
	 */
	static CORE_API FString GenerateApplicationPath( const FString& AppName, EBuildConfiguration BuildConfiguration);

	/**
	 * Get the architecture suffix, if any
	 * 
	 * Relates to UnrealArchitectureConfig.RequiresArchitectureFilenames in UnrealBuildTool
	 * 
	 * @return The platform's architecture suffix, or null if it isn't required
	 */
	static CORE_API const TCHAR* GetArchitectureSuffix();

	/**
	 * Return the prefix of dynamic library (e.g. lib)
	 *
	 * @return The prefix string.
	 * @see GetModuleExtension, GetModulesDirectory
	 */
	static CORE_API const TCHAR* GetModulePrefix();

	/**
	 * Return the extension of dynamic library
	 *
	 * @return Extension of dynamic library.
	 * @see GetModulePrefix, GetModulesDirectory
	 */
	static CORE_API const TCHAR* GetModuleExtension();

	/**
	 * Used only by platforms with DLLs, this gives the subdirectory from binaries to find the executables
	 */
	static CORE_API const TCHAR* GetBinariesSubdirectory();

	/**
	 * Used only by platforms with DLLs, this gives the full path to the main directory containing modules
	 *
	 * @return The path to the directory.
	 * @see GetModulePrefix, GetModuleExtension
	 */
	static CORE_API const FString GetModulesDirectory();

	/**
	 * Used only by platforms with DLLs, this confirms the existence of a given module.
	 * This should be called with the proper extension for modules.
	 * Returning true should ensure that GetDllHandle() will find the same file.
	 */
	static CORE_API bool ModuleExists(const FString& Filename);
	
	/**
	 * Launch a uniform resource locator (i.e. http://www.epicgames.com/unreal).
	 * This is expected to return immediately as the URL is launched by another
	 * task. The URL param must already be a valid URL. If you're looking for code 
	 * to properly escape a URL fragment, use FGenericPlatformHttp::UrlEncode.
	 */
	static CORE_API void LaunchURL( const TCHAR* URL, const TCHAR* Parms, FString* Error );

	/**
	 * Launch a uniform resource locator (i.e. http://www.epicgames.com/unreal).
	 * This is expected to return immediately as the URL is launched by another
	 * task. The URL param must already be a valid URL. The URL is passed through
	 * the filter parameter for an added measure of security if the URL is from
	 * and untrusted source. If you're looking for code to properly escape
	 * a URL fragment, use FGenericPlatformHttp::UrlEncode.
	 * 
	 * @return true if URL passed the filter and was launched, false if it was rejected by the filter.
	 */
	static CORE_API bool LaunchURLFiltered(const TCHAR* URL, const TCHAR* Parms, FString* Error, const UE::Core::FURLRequestFilter& Filter);

	/**
	 * Checks if the platform can launch a uniform resource locator (i.e. http://www.epicgames.com/unreal).
	 **/
	static CORE_API bool CanLaunchURL(const TCHAR* URL);
	
	/**
	 * Retrieves the platform-specific bundle identifier or package name of the game
	 *
	 * @return The game's bundle identifier or package name.
	 */
	static CORE_API FString GetGameBundleId();
	
	/**
	 * Creates a new process and its primary thread. The new process runs the
	 * specified executable file in the security context of the calling process.
	 * @param URL					executable name
	 * @param Parms					command line arguments
	 * @param bLaunchDetached		if true, the new process will have its own window
	 * @param bLaunchHidden			if true, the new process will be minimized in the task bar
	 * @param bLaunchReallyHidden	if true, the new process will not have a window or be in the task bar
	 * @param OutProcessId			if non-NULL, this will be filled in with the ProcessId
	 * @param PriorityModifier		-2 idle, -1 low, 0 normal, 1 high, 2 higher
	 * @param OptionalWorkingDirectory		Directory to start in when running the program, or NULL to use the current working directory
	 * @param PipeWriteChild		Optional HANDLE to pipe for redirecting output
	 * @param PipeReadChild			Optional HANDLE to pipe for redirecting input
	 * @return	The process handle for use in other process functions
	 */
	static CORE_API FProcHandle CreateProc( const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild = nullptr);
	
	/**
	 * Creates a new process and its primary thread, with separate std pipes. The new process runs the
	 * specified executable file in the security context of the calling process.
	 * @param URL					executable name
	 * @param Parms					command line arguments
	 * @param bLaunchDetached		if true, the new process will have its own window
	 * @param bLaunchHidden			if true, the new process will be minimized in the task bar
	 * @param bLaunchReallyHidden	if true, the new process will not have a window or be in the task bar
	 * @param OutProcessId			if non-NULL, this will be filled in with the ProcessId
	 * @param PriorityModifier		-2 idle, -1 low, 0 normal, 1 high, 2 higher
	 * @param OptionalWorkingDirectory		Directory to start in when running the program, or NULL to use the current working directory
	 * @param PipeWriteChild		Optional HANDLE to pipe for redirecting stdout
	 * @param PipeReadChild			Optional HANDLE to pipe for redirecting stdin
	 * @param PipeStdErrChild		Optional HANDLE to pipe for redirecting stderr
	 * @return	The process handle for use in other process functions
	 */
	static CORE_API FProcHandle CreateProc( const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild, void* PipeStdErrChild);

	/**
	 * Creates a new process using StartInfo. The new process runs the specified executable file in the security context of the calling process.
	 * @param StartInfo				The starting parameters 
	 * @return	The process Handle and Id for use in other process functions
	 */
	static CORE_API TTuple<FProcHandle, UE::HAL::EProcessId> CreateProc(const UE::HAL::FProcessStartInfo& StartInfo);
	
	/**
	 * Opens an existing process. 
	 *
	 * @param ProcessID				The process id of the process for which we want to obtain a handle.
	 * @return The process handle for use in other process functions
	 */
	static CORE_API FProcHandle OpenProcess(uint32 ProcessID);

	/**
	 * Returns true if the specified process is running 
	 *
	 * @param ProcessHandle handle returned from FPlatformProcess::CreateProc
	 * @return true if the process is still running
	 */
	static CORE_API bool IsProcRunning( FProcHandle & ProcessHandle );
	
	/**
	 * Waits for a process to stop
	 *
	 * @param ProcessHandle handle returned from FPlatformProcess::CreateProc
	 */
	static CORE_API void WaitForProc( FProcHandle & ProcessHandle );

	/**
	 * Cleans up FProcHandle after we're done with it.
	 *
	 * @param ProcessHandle handle returned from FPlatformProcess::CreateProc.
	 */
	static CORE_API void CloseProc( FProcHandle & ProcessHandle );

	/** Terminates a process
	 *
	 * @param ProcessHandle handle returned from FPlatformProcess::CreateProc
	 * @param KillTree Whether the entire process tree should be terminated.
	 */
	static CORE_API void TerminateProc( FProcHandle & ProcessHandle, bool KillTree = false );

	/** Terminates a process tree
	 *
	 * @param ProcessHandle handle returned from FPlatformProcess::CreateProc
	 * @param Predicate that returns true if the process identified by ProcessId and ApplicationName
	 *        should be terminated with its children, else that process and its children will be kept alive
	 */
	static CORE_API void TerminateProcTreeWithPredicate(
			FProcHandle& ProcessHandle,
			TFunctionRef<bool(uint32 ProcessId, const TCHAR* ApplicationName)> Predicate);

	enum class EWaitAndForkResult : uint8
	{
		Error,
		Parent,
		Child
	};

	/**
	 * Waits for process signals and forks child processes.
	 *
	 * WaitAndFork stalls the invoking process and forks child processes when signals are sent to it from an external source.
	 * Forked child processes will provide a return value of EWaitAndForkResult::Child, while the parent process
	 * will not return until IsEngineExitRequested() is true (EWaitAndForkResult::Parent) or there was an error (EWaitAndForkResult::Error)
	 * The signal the parent process expects is platform-specific (i.e. SIGRTMIN+1 on Linux). 
	 */
	static CORE_API EWaitAndForkResult WaitAndFork();

	/** Retrieves the termination status of the specified process. **/
	static CORE_API bool GetProcReturnCode( FProcHandle & ProcHandle, int32* ReturnCode );

	/** Returns true if the specified application is running */
	static CORE_API bool IsApplicationRunning( uint32 ProcessId );

	/** Returns true if the specified application is running */
	static CORE_API bool IsApplicationRunning( const TCHAR* ProcName );

	/** Returns the Name of process given by the PID.  Returns Empty string "" if PID not found. */
	static CORE_API FString GetApplicationName( uint32 ProcessId );

	/** Outputs the virtual memory usage, of the process with the specified PID */
	static CORE_API bool GetApplicationMemoryUsage(uint32 ProcessId, SIZE_T* OutMemoryUsage);

	/**
	 * Executes a process, returning the return code, stdout, and stderr. This
	 * call blocks until the process has returned.
	 * @param OutReturnCode may be 0
	 * @param OutStdOut may be 0
	 * @param OutStdErr may be 0
	 * @OptionalWorkingDirectory may be 0
	 * @OptionalbShouldEndWithParentProcess false by default. True to make sure the process is killed with the parent processor (Not Supported on all Platforms)
	 */
	static CORE_API bool ExecProcess(const TCHAR* URL, const TCHAR* Params, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr, const TCHAR* OptionalWorkingDirectory = NULL, bool bShouldEndWithParentProcess = false);

	/**
	 * Executes a process as administrator, requesting elevation as necessary. This
	 * call blocks until the process has returned.
	 */
	static CORE_API bool ExecElevatedProcess(const TCHAR* URL, const TCHAR* Params, int32* OutReturnCode);

	/**
	 * Attempt to launch the provided file name in its default external application. Similar to FPlatformProcess::LaunchURL,
	 * with the exception that if a default application isn't found for the file, the user will be prompted with
	 * an "Open With..." dialog.
	 *
	 * @param	FileName	Name of the file to attempt to launch in its default external application
	 * @param	Parms		Optional parameters to the default application
	 * @param	Verb		Optional verb to use when opening the file, if it applies for the platform.
	 * @return true if the file is launched successfully, false otherwise.
	 */
	static CORE_API bool LaunchFileInDefaultExternalApplication( const TCHAR* FileName, const TCHAR* Parms = NULL, ELaunchVerb::Type Verb = ELaunchVerb::Open, bool bPromptToOpenOnFailure = true );

	/**
	 * Attempt to "explore" the folder specified by the provided file path
	 *
	 * @param	FilePath	File path specifying a folder to explore
	 */
	static CORE_API void ExploreFolder( const TCHAR* FilePath );

#if PLATFORM_HAS_BSD_TIME 

	/** Sleep this thread for Seconds.  0.0 means release the current time slice to let other threads get some attention. Uses stats.*/
	static CORE_API void Sleep( float Seconds );
	/** Sleep this thread for Seconds.  0.0 means release the current time slice to let other threads get some attention. */
	static CORE_API void SleepNoStats( float Seconds );
	/** Sleep this thread infinitely. */
	[[noreturn]] static CORE_API void SleepInfinite();
	/** Yield this thread so another may run for a while. */
	static CORE_API void YieldThread();

#endif // PLATFORM_HAS_BSD_TIME

	/**
	* Sleep thread until condition is satisfied.
	*
	* @param	Condition	Condition to evaluate.
	* @param	SleepTime	Time to sleep
	*/
	static CORE_API void ConditionalSleep(TFunctionRef<bool()> Condition, float SleepTime = 0.0f);

	/**
	 * Creates a new event.
	 *
	 * @param bIsManualReset Whether the event requires manual reseting or not.
	 * @return A new event, or nullptr none could be created.
	 * @see GetSynchEventFromPool, ReturnSynchEventToPool
	 */
	// Message to others in the future, don't try to delete this function as it isn't exactly deprecated,
	// but it should only ever be called from TEventPool::GetEventFromPool()/FSafeRecyclableEvent::GetRawEvent()
	UE_DEPRECATED(5.0, "Please use GetSynchEventFromPool to create a new event, and ReturnSynchEventToPool to release the event.")
	static CORE_API class FEvent* CreateSynchEvent(bool bIsManualReset = false);

	/**
	 * Gets an event from the pool or creates a new one if necessary.
	 *
	 * @param bIsManualReset Whether the event requires manual reseting or not.
	 * @return An event, or nullptr none could be created.
	 * @see CreateSynchEvent, ReturnSynchEventToPool
	 */
	static CORE_API class FEvent* GetSynchEventFromPool(bool bIsManualReset = false);

	/**
	 * Deletes all the recycled sync events contained by the pools
	 */
	static CORE_API void FlushPoolSyncEvents();

	/**
	 * Returns an event to the pool.
	 *
	 * @param Event The event to return.
	 * @see CreateSynchEvent, GetSynchEventFromPool
	 */
	static CORE_API void ReturnSynchEventToPool(FEvent* Event);

	/**
	 * Creates the platform-specific runnable thread. This should only be called from FRunnableThread::Create.
	 *
	 * @return The newly created thread
	 */
	static CORE_API class FRunnableThread* CreateRunnableThread();

	/**
	 * Closes an anonymous pipe.
	 *
	 * @param ReadPipe The handle to the read end of the pipe.
	 * @param WritePipe The handle to the write end of the pipe.
	 * @see CreatePipe, ReadPipe
	 */
	static CORE_API void ClosePipe( void* ReadPipe, void* WritePipe );

	/**
	 * Creates a writable anonymous pipe.
	 *
	 * Anonymous pipes can be used to capture and/or redirect STDOUT and STDERROR of a process.
	 * The pipe created by this method can be passed into CreateProc as Write
	 *
	 * @param ReadPipe Will hold the handle to the read end of the pipe.
	 * @param WritePipe Will hold the handle to the write end of the pipe.
	 * @parm bWritePipeLocal indicates that the write pipe end will be used locally, instead of the read pipe
	 * @return true on success, false otherwise.
	 * @see ClosePipe, ReadPipe
	 */
	static CORE_API bool CreatePipe(void*& ReadPipe, void*& WritePipe, bool bWritePipeLocal = false);

	/**
	 * Reads all pending data from an anonymous pipe, such as STDOUT or STDERROR of a process.
	 *
	 * @param Pipe The handle to the pipe to read from.
	 * @return A string containing the read data.
	 * @see ClosePipe, CreatePipe
	 */
	static CORE_API FString ReadPipe( void* ReadPipe );

	/**
	 * Reads all pending data from an anonymous pipe, such as STDOUT or STDERROR of a process.
	 *
	 * @param Pipe The handle to the pipe to read from.
	 * @param Output The data read.
	 * @return true if successful (i.e. any data was read)
	 * @see ClosePipe, CreatePipe
	 */
	static CORE_API bool ReadPipeToArray(void* ReadPipe, TArray<uint8> & Output);

	/**
	* Sends the message to process through pipe
	*
	* @param WritePipe Pipe for writing.
	* @param Message The message to be written.
	* @param OutWritten Optional parameter to know how much of the string written.
	* @return True if all bytes written successfully.
	* @see CreatePipe, ClosePipe, ReadPipe
	*/
	static CORE_API bool WritePipe(void* WritePipe, const FString& Message, FString* OutWritten = nullptr);

	/**
	* Sends data to process through pipe
	*
	* @param WritePipe Pipe for writing.
	* @param Data The data to be written.
	* @param DataLength how many bytes to write.
	* @param OutDataLength Optional parameter to know how many bytes had been written.
	* @return True if all bytes written successfully.
	* @see CreatePipe, ClosePipe, ReadPipe
	*/
	static CORE_API bool WritePipe(void* WritePipe, const uint8* Data, const int32 DataLength, int32* OutDataLength = nullptr);


	/**
	 * Gets whether this platform can use multiple threads.
	 *
	 * @return true if the platform can use multiple threads, false otherwise.
	 */
	static CORE_API bool SupportsMultithreading();

	/**
	 * Creates or opens an interprocess synchronization object.
	 *
	 * @param Name name (so we can use it across processes).
	 * @param bCreate If true, the function will try to create, otherwise will try to open existing.
	 * @param MaxLocks Maximum amount of locks that the semaphore can have (pass 1 to make it act as mutex).
	 * @return Pointer to heap allocated semaphore object. Caller is responsible for deletion.
	 */
	static CORE_API FSemaphore* NewInterprocessSynchObject(const FString& Name, bool bCreate, uint32 MaxLocks = 1);

	/**
	 * Creates or opens an interprocess synchronization object.
	 *
	 * @param Name name (so we can use it across processes).
	 * @param bCreate If true, the function will try to create, otherwise will try to open existing.
	 * @param MaxLocks Maximum amount of locks that the semaphore can have (pass 1 to make it act as mutex).
	 * @return Pointer to heap allocated semaphore object. Caller is responsible for deletion.
	 */
	static CORE_API FSemaphore* NewInterprocessSynchObject(const TCHAR* Name, bool bCreate, uint32 MaxLocks = 1);

	/**
	 * Deletes an interprocess synchronization object.
	 *
	 * @param Object object to destroy.
	 */
	static CORE_API bool DeleteInterprocessSynchObject(FSemaphore * Object);

	/**
	 * Makes process run as a system service (daemon), i.e. detaches it from whatever user session it was initially run from.
	 *
	 * @return true if successful, false otherwise.
	 */
	static CORE_API bool Daemonize();

	/**
	 * Checks if we're the first instance. An instance can become first if the previous first instance quits before it.
	 */
	static CORE_API bool IsFirstInstance();

	/**
	 * Stop reporting that we are the first instance
	 */
	static CORE_API void CeaseBeingFirstInstance();

	/**
	 * Tears down allocated process resources.
	 */
	static CORE_API void TearDown();

	/**
	 * force skip calling FThreadStats::WaitForStats()
	 */
	static bool SkipWaitForStats() { return false; }

	/**
	 * Queries the memory usage of the process. Returns whether the operation is supported and succeeded.
	 */
	static bool TryGetMemoryUsage(FProcHandle& ProcessHandle, FPlatformProcessMemoryStats& OutStats) { return false; }

	/**
	 * specifies the thread to use for UObject reference collection
	 */
	static CORE_API ENamedThreads::Type GetDesiredThreadForUObjectReferenceCollector();

	/**
	 * allows a platform to override the threading configuration for reference collection
	 */
	static CORE_API void ModifyThreadAssignmentForUObjectReferenceCollector( int32& NumThreads, int32& NumBackgroundThreads, ENamedThreads::Type& NormalThreadName, ENamedThreads::Type& BackgroundThreadName );

	/** 
	 * Create a GenericProcessSentinel, or a subclass, object
	 */
	static CORE_API TSharedPtr<IProcessSentinel> CreateProcessSentinelObject();

	/** 
	 * Create a GenericProcessSentinel, or a subclass, object, and then uses it to create the actual sentinel file on disk
	 */
	static CORE_API TSharedPtr<IProcessSentinel> CreateProcessSentinel(const TCHAR* Name, const FString& Contents);	

	/**
	 * Tells the processor to pause for implementation-specific amount of time. Is used for spin-loops to improve the speed at 
	 * which the code detects the release of the lock and power-consumption.
	 */
	static inline void Yield()
	{
#if PLATFORM_USE_SSE2_FOR_THREAD_YIELD
		_mm_pause();
#elif PLATFORM_CPU_ARM_FAMILY
#	if defined(_MSC_VER)
		__isb(0);
#	else
		__asm__ __volatile__("isb");
#	endif
#else
		// the platform with other architectures must override this to not have this function be called
		unimplemented();
#endif
	}

	/**
	* Tells the processor to pause for at least the amount of cycles given. Is used for spin-loops to improve the speed at 
	* which the code detects the release of the lock and power-consumption.
	*/
	static inline void YieldCycles(uint64 Cycles)
	{
#if PLATFORM_CPU_X86_FAMILY
		auto ReadCycleCounter = []()
		{
#if defined(_MSC_VER)
			return __rdtsc();
#elif PLATFORM_APPLE
			return mach_absolute_time();
#elif __has_builtin(__builtin_readcyclecounter)
			return __builtin_readcyclecounter();
#else
			// the platform with other architectures must override this to not have this function be called
			unimplemented();
#endif
		};

		uint64 start = ReadCycleCounter();
		//some 32bit implementations return 0 for __builtin_readcyclecounter just to be on the safe side we protect against this.
		Cycles = start != 0 ? Cycles : 0;

#if PLATFORM_WINDOWS
		if (FPlatformMisc::HasTimedPauseCPUFeature())
		{
			uint64 PauseCycles = ReadCycleCounter() + Cycles;
#if defined(_MSC_VER)
			_tpause(0, PauseCycles);
#elif __has_builtin(__builtin_ia32_tpause)
			__builtin_ia32_tpause(0, (uint32)(PauseCycles >> 32), (uint32)PauseCycles);
#else
#			error Unsupported architecture!
#endif
		}
		else
#endif
		{
			do
			{
				Yield();
			} while ((ReadCycleCounter() - start) < Cycles);
		}

#else
		// We can't read cycle counter from user mode on these platform
		for (uint64 i = 0; i < Cycles; i++)
		{
#if PLATFORM_CPU_ARM_FAMILY
#	if defined(_MSC_VER)
			__yield();
#	else
			__builtin_arm_yield();
#	endif
#else
#			error "Not implemented on this platform"
#endif
		}
#endif
	}
};
