// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformMisc.h"


/**
 * Base class for platform properties.
 *
 * These are shared between:
 *     the runtime platform - via FPlatformProperties
 *     the target platforms - via ITargetPlatform
 */
struct FGenericPlatformProperties
{
	/**
	 * Gets the platform's physics format.
	 *
	 * @return The physics format name.
	 */
	static constexpr UE_FORCEINLINE_HINT const char* GetPhysicsFormat()
	{
		return "Chaos";
	}

	/**
	 * Gets whether this platform has Editor-only data.
	 *
	 * @return true if the platform has Editor-only data, false otherwise.
	 */
	static constexpr UE_FORCEINLINE_HINT bool HasEditorOnlyData()
	{
		return WITH_EDITORONLY_DATA;
	}

	/**
	 * Gets the name of this platform when loading INI files. Defaults to PlatformName.
	 *
	 * Note: MUST be implemented per platform.
	 *
	 * @return Platform name.
	 */
	static const char* IniPlatformName();

	/**
	 * Gets whether this is a game only platform.
	 *
	 * @return true if this is a game only platform, false otherwise.
	 */
	static constexpr UE_FORCEINLINE_HINT bool IsGameOnly()
	{
		return UE_GAME;
	}

	/**
	 * Gets whether this is a server only platform.
	 *
	 * @return true if this is a server only platform, false otherwise.
	 */
	static constexpr UE_FORCEINLINE_HINT bool IsServerOnly()
	{
		return UE_SERVER;
	}

	/**
	 * Gets whether this is a client only (no capability to run the game without connecting to a server) platform.
	 *
	 * @return true if this is a client only platform, false otherwise.
	 */
	static constexpr UE_FORCEINLINE_HINT bool IsClientOnly()
	{
		return !WITH_SERVER_CODE;
	}

	/**
	 *	Gets whether this was a monolithic build or not
	 */
	static constexpr UE_FORCEINLINE_HINT bool IsMonolithicBuild()
	{
		return IS_MONOLITHIC;
	}

	/**
	 *	Gets whether this was a merged modules build or not
	 */
	static constexpr UE_FORCEINLINE_HINT bool IsMergedModulesBuild()
	{
		return UE_MERGED_MODULES;
	}

	/**
	 *	Gets whether this was a program or not
	 */
	static constexpr UE_FORCEINLINE_HINT bool IsProgram()
	{
		return IS_PROGRAM;
	}

	/**
	 * Gets whether this is a Little Endian platform.
	 *
	 * @return true if the platform is Little Endian, false otherwise.
	 */
	static constexpr UE_FORCEINLINE_HINT bool IsLittleEndian()
	{
		return true;
	}

	/**
	 * Gets the name of this platform
	 *
	 * Note: MUST be implemented per platform.
	 *
	 * @return Platform Name.
	 */
	static UE_FORCEINLINE_HINT const char* PlatformName();

	/**
	  * Get the name of the hardware variant of the current platform.
	  * 
	  * Most platforms don't need to provide overrides for this member. This member is intended to be used by the few
	  * which come in different hardware flavours or which may operate in different runtime modes.
	  * 
	  * @return Name of the platform variant.
	  */
	static UE_FORCEINLINE_HINT const char* PlatformVariantName()
	{
		return "";
	}

	/**
	 * Checks whether this platform requires cooked data.
	 *
	 * @return true if cooked data is required, false otherwise.
	 */
	static constexpr UE_FORCEINLINE_HINT bool RequiresCookedData()
	{
		return !HasEditorOnlyData();
	}

	/**
	* Checks whether shipped data on this platform is secure, and doesn't require extra encryption/signing to protect it.
	*
	* @return true if packaged data is considered secure, false otherwise.
	*/
	static constexpr UE_FORCEINLINE_HINT bool HasSecurePackageFormat()
	{
		return false;
	}

	/**
	 * Checks whether this platform requires user credentials (typically server platforms).
	 *
	 * @return true if this platform requires user credentials, false otherwise.
	 */
	static constexpr UE_FORCEINLINE_HINT bool RequiresUserCredentials()
	{
		return false;
	}

	/**
	 * Checks whether the specified build target is supported.
	 *
	 * @param TargetType The build target to check.
	 * @return true if the build target is supported, false otherwise.
	 */
	static UE_FORCEINLINE_HINT bool SupportsBuildTarget(EBuildTargetType TargetType)
	{
		return true;
	}

	/**
	 * Returns true if platform supports the AutoSDK system
	 */
	static constexpr UE_FORCEINLINE_HINT bool SupportsAutoSDK()
	{
		return false;
	}

	/**
	 * Gets whether this platform supports gray scale sRGB texture formats.
	 *
	 * @return true if gray scale sRGB texture formats are supported.
	 */
	static constexpr UE_FORCEINLINE_HINT bool SupportsGrayscaleSRGB()
	{
		return true;
	}

	/**
	 * Checks whether this platforms supports running multiple game instances on a single device.
	 *
	 * @return true if multiple instances are supported, false otherwise.
	 */
	static constexpr UE_FORCEINLINE_HINT bool SupportsMultipleGameInstances()
	{
		return false;
	}

	/**
	 * Gets whether this platform supports windowed mode rendering.
	 *
	 * @return true if windowed mode is supported.
	 */
	static constexpr UE_FORCEINLINE_HINT bool SupportsWindowedMode()
	{
		return false;
	}

	/**
	 * Whether this platform wants to allow framerate smoothing or not.
	 */
	static constexpr UE_FORCEINLINE_HINT bool AllowsFramerateSmoothing()
	{
		return true;
	}

	/**
	 * Whether this platform supports streaming audio
	 */
	static constexpr UE_FORCEINLINE_HINT bool SupportsAudioStreaming()
	{
		return false;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsHighQualityLightmaps()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsLowQualityLightmaps()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsDistanceFieldShadows()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsDistanceFieldAO()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsTextureStreaming()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMeshLODStreaming()
	{
		return false;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMemoryMappedFiles()
	{
		return false;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMemoryMappedAudio()
	{
		return false;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMemoryMappedAnimation()
	{
		return false;
	}

	static constexpr UE_FORCEINLINE_HINT int64 GetMemoryMappingAlignment()
	{
		return 0;
	}

	// Guaranteed virtual memory alignment on a given platform, regardless of a specific device
	static constexpr UE_FORCEINLINE_HINT int GetMaxSupportedVirtualMemoryAlignment()
	{
		return 4096;
	}
	
	static constexpr UE_FORCEINLINE_HINT bool SupportsRayTracing()
	{
		return false;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsLumenGI()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsHardwareLZDecompression()
	{
		return false;
	}

	/**
	 * Gets whether user settings should override the resolution or not
	 */
	static constexpr UE_FORCEINLINE_HINT bool HasFixedResolution()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMinimize()
	{
		return false;
	}

	// Whether the platform allows an application to quit to the OS
	static constexpr UE_FORCEINLINE_HINT bool SupportsQuit()
	{
		return false;
	}

	// Whether the platform allows the call stack to be dumped during an assert
	static constexpr UE_FORCEINLINE_HINT bool AllowsCallStackDumpDuringAssert()
	{
		return IsProgram();
	}

	// If this platform wants to replace Zlib with a platform-specific version, set the name of the compression format 
	// plugin (matching its GetCompressionFormatName() function) in an override of this function
	static constexpr UE_FORCEINLINE_HINT const char* GetZlibReplacementFormat()
	{
		return nullptr;
	}
	 
	// Whether the platform requires an original release version to make a patch
	static constexpr UE_FORCEINLINE_HINT bool RequiresOriginalReleaseVersionForPatch()
	{
		return false;
	}

	// Whether the platform has non-trivial patching requirements, where regular workflows may result in invalid patches
	static constexpr UE_FORCEINLINE_HINT bool HasNonTrivialPatchGeneration()
	{
		return false;
	}
};
