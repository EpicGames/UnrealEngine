// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "UObject/NameTypes.h"

class FName;
struct FGenericCrashContext;
struct FProcHandle;
struct FProgramCounterSymbolInfoEx;

/**
 * This is used to capture all of the module information needed to load pdb's.
 * @todo, the memory profiler should not be using this as platform agnostic
 */
struct FStackWalkModuleInfo
{
	uint64 BaseOfImage;
	uint32 ImageSize;
	uint32 TimeDateStamp;
	TCHAR ModuleName[32];
	TCHAR ImageName[256];
	TCHAR LoadedImageName[256];
	uint32 PdbSig;
	uint32 PdbAge;
	struct
	{
		unsigned long  Data1;
		unsigned short Data2;
		unsigned short Data3;
		unsigned char  Data4[8];
	} PdbSig70;
};

/**
 * Symbol information associated with a program counter. ANSI version.
 */
struct FProgramCounterSymbolInfo final
{
	enum
	{
		/** Length of the string used to store the symbol's names, including the trailing character. */
		MAX_NAME_LENGTH = 1024,
	};

	/** Module name. */
	ANSICHAR	ModuleName[MAX_NAME_LENGTH];

	/** Function name. */
	ANSICHAR	FunctionName[MAX_NAME_LENGTH];

	/** Filename. */
	ANSICHAR	Filename[MAX_NAME_LENGTH];

	/** Line number in file. */
	int32		LineNumber;

	/** Symbol displacement of address.	*/
	int32		SymbolDisplacement;

	/** Program counter offset into module. */
	uint64		OffsetInModule;

	/** Program counter. */
	uint64		ProgramCounter;

	/** Default constructor. */
	CORE_API FProgramCounterSymbolInfo();
};

struct FProgramCounterSymbolInfoEx
{
	/** Module name. */
	FString	ModuleName;

	/** Function name. */
	FString	FunctionName;

	/** Filename. */
	FString	Filename;

	/** Line number in file. */
	uint32	LineNumber;

	/** Symbol displacement of address.	*/
	uint64	SymbolDisplacement;

	/** Program counter offset into module. */
	uint64	OffsetInModule;

	/** Program counter. */
	uint64	ProgramCounter;

	/** Default constructor. */
	CORE_API FProgramCounterSymbolInfoEx(FString InModuleName = "", FString InFunctionName = "", FString InFilename = "", uint32 InLineNumber = 0, uint64 InSymbolDisplacement = 0, uint64 InOffsetInModule = 0, uint64 InProgramCounter = 0);
};

/**
 * Generic implementation for most platforms
 */
struct FGenericPlatformStackWalk
{
	typedef FGenericPlatformStackWalk Base;

	struct EStackWalkFlags
	{
		enum
		{
			/** Default value (empty set of flags). */
			AccurateStackWalk				=	0,

			/** Used when preferring speed over more information. Platform-specific, may be ignored, or may result in non-symbolicated callstacks or missing some other information. */
			FastStackWalk					=	(1 << 0),

			/** This is a set of flags that will be passed when unwinding the callstack for ensure(). */
			FlagsUsedWhenHandlingEnsure		=	(FastStackWalk)
		};
	};

	/**
	* Initializes options related to stack walking from ini, i.e. how detailed the stack walking should be, performance settings etc.
	*/
	static CORE_API void Init();

	/**
	 * Initializes stack traversal and symbol. Must be called before any other stack/symbol functions. Safe to reenter.
	 */
	static bool InitStackWalking()
	{
		return 1;
	}

	/**
	 * Like InitStackWalking but initializes stack walking for another process. 
	 * @param	Process		Process that is going to be queried
	 * @return				True if successful, false otherwise.
	 */
	static bool InitStackWalkingForProcess(const FProcHandle& Process)
	{
		return false;
	}

	/**
	 * Converts the passed in program counter address to a human readable string and appends it to the passed in one.
	 * @warning: The code assumes that HumanReadableString is large enough to contain the information.
	 * @warning: Please, do not override this method. Can't be modified or altered without notice.
	 * 
	 * This method is the same for all platforms to simplify parsing by the crash processor.
	 * 
	 * Example formatted line:
	 *
	 * UE4Editor_Core!FOutputDeviceWindowsError::Serialize() (0xddf1bae5) + 620 bytes [\engine\source\runtime\core\private\windows\windowsplatformoutputdevices.cpp:110]
	 * ModuleName!FunctionName (ProgramCounter) + offset bytes [StrippedFilepath:LineNumber]
	 *
	 * @param	CurrentCallDepth		Depth of the call, if known (-1 if not - note that some platforms may not return meaningful information in the latter case)
	 * @param	ProgramCounter			Address to look symbol information up for
	 * @param	HumanReadableString		String to concatenate information with
	 * @param	HumanReadableStringSize size of string in characters
	 * @param	Context					Pointer to crash context, if any
	 * @return	true if the symbol was found, otherwise false
	 */ 
	static CORE_API bool ProgramCounterToHumanReadableString( int32 CurrentCallDepth, uint64 ProgramCounter, ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, FGenericCrashContext* Context = nullptr );

	/**
	 * Converts the passed in symbol information to a human readable string and appends it to the passed in one.
	 * @warning: The code assumes that HumanReadableString is large enough to contain the information.
	 * @warning: Please, do not override this method. Can't be modified or altered without notice.
	 * 
	 * This method is the same for all platforms to simplify parsing by the crash processor.
	 * 
	 * Example formatted line:
	 *
	 * UE4Editor_Core!FOutputDeviceWindowsError::Serialize() (0xddf1bae5) + 620 bytes [\engine\source\runtime\core\private\windows\windowsplatformoutputdevices.cpp:110]
	 * ModuleName!FunctionName (ProgramCounter) + offset bytes [StrippedFilepath:LineNumber]
	 *
	 * @param	SymbolInfo				Symbol information
	 * @param	HumanReadableString		String to concatenate information with
	 * @param	HumanReadableStringSize size of string in characters
	 * @param	Context					Pointer to crash context, if any
	 * @return	true if the symbol was found, otherwise false
	 */ 
	static CORE_API bool SymbolInfoToHumanReadableString( const FProgramCounterSymbolInfo& SymbolInfo, ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize );

	/** Same as above, but can be used with external applications. */
	static CORE_API bool SymbolInfoToHumanReadableStringEx( const FProgramCounterSymbolInfoEx& SymbolInfo, FString& out_HumanReadableString );

	/**
	 * Converts the passed in program counter address to a symbol info struct, filling in module and filename, line number and displacement.
	 * @warning: The code assumes that the destination strings are big enough
	 *
	 * @param	ProgramCounter		Address to look symbol information up for
	 * @param	out_SymbolInfo		Symbol information associated with program counter
	 */
	static void ProgramCounterToSymbolInfo( uint64 ProgramCounter, FProgramCounterSymbolInfo& out_SymbolInfo)
	{
		out_SymbolInfo.ProgramCounter = ProgramCounter;
	}

	/** Same as above, but can be used with external applications since it doesn't re-initialize the active process. */
	static void ProgramCounterToSymbolInfoEx( uint64 ProgramCounter, FProgramCounterSymbolInfoEx& out_SymbolInfo)
	{
		out_SymbolInfo.ProgramCounter = ProgramCounter;
	}

	/**
	 * Capture a stack backtrace and optionally use the passed in exception pointers.
	 *
	 * @param	BackTrace			[out] Pointer to array to take backtrace
	 * @param	MaxDepth			Entries in BackTrace array
	 * @param	Context				Optional thread context information
	 */
	static CORE_API uint32 CaptureStackBackTrace( uint64* BackTrace, uint32 MaxDepth, void* Context = nullptr );

	/**
	 * Capture a stack backtrace for a specific thread.
	 *
	 * @param	ThreadId			ID of the thread to trace
	 * @param	BackTrace			[out] Pointer to array to take backtrace
	 * @param	MaxDepth			Entries in BackTrace array
	 * @param	Context				Optional thread context information
	 */
	static CORE_API uint32 CaptureThreadStackBackTrace(uint64 ThreadId, uint64* BackTrace, uint32 MaxDepth, void* Context = nullptr);

	/**
	 * Walks the stack and appends the human readable string to the passed in one.
	 * @warning: The code assumes that HumanReadableString is large enough to contain the information.
	 *
	 * @param	HumanReadableString	String to concatenate information with
	 * @param	HumanReadableStringSize size of string in characters
	 * @param	IgnoreCount			Number of stack entries to ignore (some are guaranteed to be in the stack walking code)
	 * @param	Context				Optional thread context information
	 */ 
	CORE_API FORCENOINLINE static void StackWalkAndDump( ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, void* Context = nullptr );  // FORCENOINLINE so it can be counted during StackTrace

	/**
	 * Walks the stack and appends the human readable string to the passed in one.
	 * @warning: The code assumes that HumanReadableString is large enough to contain the information.
	 *
	 * @param	HumanReadableString	String to concatenate information with
	 * @param	HumanReadableStringSize size of string in characters
	 * @param	ProgramCounter		Instruction address that should be at the top of the stack
	 * @param	Context				Optional thread context information
	 */ 
	static CORE_API void StackWalkAndDump( ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, void* ProgramCounter, void* Context = nullptr );

	/**
	 * Walks the stack and updates the Stack array with the symbol information for each line in the stack.
	 *
	 * @param	IgnoreCount			Number of stack entries to ignore (some are guaranteed to be in the stack walking code)
	 * @param	MaxDepth			The maximum depth to trace, can't be more than 100, offset from IgnoreCount.
	 * @param	Context				Optional thread context information
	 * 
	 * @return	The stack of symbols to return.
	 */ 
	static CORE_API TArray<FProgramCounterSymbolInfo> GetStack(int32 IgnoreCount, int32 MaxDepth = 100, void* Context = nullptr);

	/**
	* Walks the stack for the specified thread and appends the human readable string to the passed in one.
	* @warning: The code assumes that HumanReadableString is large enough to contain the information.
	*
	* @param	HumanReadableString	String to concatenate information with
	* @param	HumanReadableStringSize size of string in characters
	* @param	IgnoreCount			Number of stack entries to ignore (some are guaranteed to be in the stack walking code)
	* @param	ThreadId				ThreadId to walk the strack for.
	*/
	static void ThreadStackWalkAndDump(ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, uint32 ThreadId)
	{
	}

	/**
	 * Walks the stack and appends the human readable string to the passed in one.
	 * @warning: The code assumes that HumanReadableString is large enough to contain the information.
	 *
	 * @param	HumanReadableString	String to concatenate information with
	 * @param	HumanReadableStringSize size of string in characters
	 * @param	IgnoreCount			Number of stack entries to ignore (some are guaranteed to be in the stack walking code)
	 * @param   Flags				Used to pass additional information (see StackWalkFlags)
	 * @param	Context				Optional thread context information
	 */ 
	static CORE_API void StackWalkAndDumpEx( ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, uint32 Flags, void* Context = nullptr );
	/**
	 * Walks the stack and appends the human readable string to the passed in one.
	 * @warning: The code assumes that HumanReadableString is large enough to contain the information.
	 *
	 * @param	HumanReadableString	String to concatenate information with
	 * @param	HumanReadableStringSize size of string in characters
	 * @param	ProgramCounter		Instruction address that should be at the top of the stack
	 * @param   Flags				Used to pass additional information (see StackWalkFlags)
	 * @param	Context				Optional thread context information
	 */ 
	static CORE_API void StackWalkAndDumpEx( ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, void* ProgramCounter, uint32 Flags, void* Context = nullptr );

	/**
	 * Returns the number of modules loaded by the currently running process.
	 */
	FORCEINLINE static int32 GetProcessModuleCount()
	{
		return 0;
	}

	/**
	 * Gets the signature for every module loaded by the currently running process.
	 *
	 * @param	ModuleSignatures		An array to retrieve the module signatures.
	 * @param	ModuleSignaturesSize	The size of the array pointed to by ModuleSignatures.
	 *
	 * @return	The number of modules copied into ModuleSignatures
	 */
	FORCEINLINE static int32 GetProcessModuleSignatures(FStackWalkModuleInfo *ModuleSignatures, const int32 ModuleSignaturesSize)
	{
		return 0;
	}

	/**
	 * Gets the meta-data associated with all symbols of this target.
	 * This may include things that are needed to perform further offline processing of symbol information (eg, the source binary).
	 *
	 * @return	A map containing the meta-data (if any).
	 */
	static CORE_API TMap<FName, FString> GetSymbolMetaData();

	/** Converts virtual path to local and write local path to Dest. If no virtual path if sound, Source will be written */
	static CORE_API void CopyVirtualPathToLocal(char* Dest, int32 DestCapacity, const char* Source);

protected:

	/** Returns true if non-monolithic builds should produce full callstacks in the log (and load all debug symbols) */
	static CORE_API bool WantsDetailedCallstacksInNonMonolithicBuilds();

};
