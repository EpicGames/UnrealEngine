// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Optimized locale  and CRT independent case-insensitive string comparisons
 *
 * Only considers ASCII character casing, i.e. C locale semantics
 *
 * @return	Zero if strings are equal, greater than zero if first string is
 *			greater than the second one and less than zero otherwise.
 */
struct FGenericPlatformStricmp
{
	CORE_API static int32 Stricmp(const ANSICHAR* String1, const ANSICHAR* String2);
	CORE_API static int32 Stricmp(const WIDECHAR* String1, const WIDECHAR* String2);
	CORE_API static int32 Stricmp(const UTF8CHAR* String1, const UTF8CHAR* String2);
	CORE_API static int32 Stricmp(const UTF16CHAR* String1, const UTF16CHAR* String2);
	CORE_API static int32 Stricmp(const UTF32CHAR* String1, const UTF32CHAR* String2);

	CORE_API static int32 Stricmp(const ANSICHAR* String1, const WIDECHAR* String2);
	CORE_API static int32 Stricmp(const ANSICHAR* String1, const UTF8CHAR* String2);
	CORE_API static int32 Stricmp(const ANSICHAR* String1, const UTF16CHAR* String2);
	CORE_API static int32 Stricmp(const ANSICHAR* String1, const UTF32CHAR* String2);
	CORE_API static int32 Stricmp(const WIDECHAR* String1, const ANSICHAR* String2);
	CORE_API static int32 Stricmp(const UTF8CHAR* String1, const ANSICHAR* String2);
	CORE_API static int32 Stricmp(const UTF16CHAR* String1, const ANSICHAR* String2);
	CORE_API static int32 Stricmp(const UTF32CHAR* String1, const ANSICHAR* String2);

	CORE_API static int32 Strnicmp(const ANSICHAR* String1, const ANSICHAR* String2, SIZE_T Count);
	CORE_API static int32 Strnicmp(const WIDECHAR* String1, const WIDECHAR* String2, SIZE_T Count);
	CORE_API static int32 Strnicmp(const UTF8CHAR* String1, const UTF8CHAR* String2, SIZE_T Count);
	CORE_API static int32 Strnicmp(const UTF16CHAR* String1, const UTF16CHAR* String2, SIZE_T Count);
	CORE_API static int32 Strnicmp(const UTF32CHAR* String1, const UTF32CHAR* String2, SIZE_T Count);

	CORE_API static int32 Strnicmp(const ANSICHAR* String1, const WIDECHAR* String2, SIZE_T Count);
	CORE_API static int32 Strnicmp(const ANSICHAR* String1, const UTF8CHAR* String2, SIZE_T Count);
	CORE_API static int32 Strnicmp(const ANSICHAR* String1, const UTF16CHAR* String2, SIZE_T Count);
	CORE_API static int32 Strnicmp(const ANSICHAR* String1, const UTF32CHAR* String2, SIZE_T Count);
	CORE_API static int32 Strnicmp(const WIDECHAR* String1, const ANSICHAR* String2, SIZE_T Count);
	CORE_API static int32 Strnicmp(const UTF8CHAR* String1, const ANSICHAR* String2, SIZE_T Count);
	CORE_API static int32 Strnicmp(const UTF16CHAR* String1, const ANSICHAR* String2, SIZE_T Count);
	CORE_API static int32 Strnicmp(const UTF32CHAR* String1, const ANSICHAR* String2, SIZE_T Count);
};
