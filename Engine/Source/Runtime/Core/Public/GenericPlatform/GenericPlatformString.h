// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformStricmp.h"
#include "Templates/EnableIf.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "Traits/IsCharEncodingSimplyConvertibleTo.h"
#include "Traits/IsCharType.h"
#include "Traits/IsFixedWidthCharEncoding.h"

#include <type_traits>
#include <stdarg.h>

namespace UE::Core::Private
{
	// The Dest parameter is just used for overload resolution
	CORE_API int32 GetConvertedLength(const UTF8CHAR* Dest, const WIDECHAR*  Src);
	CORE_API int32 GetConvertedLength(const UTF8CHAR* Dest, const WIDECHAR*  Src, int32 SrcLen);
	CORE_API int32 GetConvertedLength(const UTF8CHAR* Dest, const UCS2CHAR*  Src);
	CORE_API int32 GetConvertedLength(const UTF8CHAR* Dest, const UCS2CHAR*  Src, int32 SrcLen);
	CORE_API int32 GetConvertedLength(const UTF8CHAR* Dest, const UTF32CHAR* Src);
	CORE_API int32 GetConvertedLength(const UTF8CHAR* Dest, const UTF32CHAR* Src, int32 SrcLen);
	CORE_API int32 GetConvertedLength(const ANSICHAR* Dest, const UTF8CHAR*  Src);
	CORE_API int32 GetConvertedLength(const ANSICHAR* Dest, const UTF8CHAR*  Src, int32 SrcLen);
	CORE_API int32 GetConvertedLength(const WIDECHAR* Dest, const UTF8CHAR*  Src);
	CORE_API int32 GetConvertedLength(const WIDECHAR* Dest, const UTF8CHAR*  Src, int32 SrcLen);
	CORE_API int32 GetConvertedLength(const UCS2CHAR* Dest, const UTF8CHAR*  Src);
	CORE_API int32 GetConvertedLength(const UCS2CHAR* Dest, const UTF8CHAR*  Src, int32 SrcLen);

	CORE_API UTF8CHAR* Convert(UTF8CHAR* Dest, int32 DestLen, const WIDECHAR*  Src);
	CORE_API UTF8CHAR* Convert(UTF8CHAR* Dest, int32 DestLen, const WIDECHAR*  Src, int32 SrcLen);
	CORE_API UTF8CHAR* Convert(UTF8CHAR* Dest, int32 DestLen, const UCS2CHAR*  Src);
	CORE_API UTF8CHAR* Convert(UTF8CHAR* Dest, int32 DestLen, const UCS2CHAR*  Src, int32 SrcLen);
	CORE_API UTF8CHAR* Convert(UTF8CHAR* Dest, int32 DestLen, const UTF32CHAR* Src);
	CORE_API UTF8CHAR* Convert(UTF8CHAR* Dest, int32 DestLen, const UTF32CHAR* Src, int32 SrcLen);
	CORE_API ANSICHAR* Convert(ANSICHAR* Dest, int32 DestLen, const UTF8CHAR*  Src);
	CORE_API ANSICHAR* Convert(ANSICHAR* Dest, int32 DestLen, const UTF8CHAR*  Src, int32 SrcLen);
	CORE_API WIDECHAR* Convert(WIDECHAR* Dest, int32 DestLen, const UTF8CHAR*  Src);
	CORE_API WIDECHAR* Convert(WIDECHAR* Dest, int32 DestLen, const UTF8CHAR*  Src, int32 SrcLen);
	CORE_API UCS2CHAR* Convert(UCS2CHAR* Dest, int32 DestLen, const UTF8CHAR*  Src);
	CORE_API UCS2CHAR* Convert(UCS2CHAR* Dest, int32 DestLen, const UTF8CHAR*  Src, int32 SrcLen);
}

// These will be moved inside GenericPlatformString.cpp when the platform layer handles UTF-16
// instead of StringConv.h.
#define HIGH_SURROGATE_START_CODEPOINT    ((uint16)0xD800)
#define HIGH_SURROGATE_END_CODEPOINT      ((uint16)0xDBFF)
#define LOW_SURROGATE_START_CODEPOINT     ((uint16)0xDC00)
#define LOW_SURROGATE_END_CODEPOINT       ((uint16)0xDFFF)
#define ENCODED_SURROGATE_START_CODEPOINT ((uint32)0x10000)
#define ENCODED_SURROGATE_END_CODEPOINT   ((uint32)0x10FFFF)

#define UNICODE_BOGUS_CHAR_CODEPOINT '?'
static_assert(sizeof(UNICODE_BOGUS_CHAR_CODEPOINT) <= sizeof(ANSICHAR) && (UNICODE_BOGUS_CHAR_CODEPOINT) >= 32 && (UNICODE_BOGUS_CHAR_CODEPOINT) <= 127, "The Unicode Bogus character point is expected to fit in a single ANSICHAR here");

/**
 * Generic string implementation for most platforms
 */
struct FGenericPlatformString : public FGenericPlatformStricmp
{
	/**
	 * Tests whether a particular codepoint can be converted to the destination encoding.
	 *
	 * @param Ch The character to test.
	 * @return True if Ch can be encoded as a DestEncoding.
	 */
	template <typename DestEncoding, typename SourceEncoding>
	static constexpr bool CanConvertCodepoint(SourceEncoding Codepoint)
	{
		// It is assumed that the incoming codepoint is already valid and we're only testing if it can be converted to DestEncoding.

		static_assert(TIsCharType<SourceEncoding>::Value, "Source encoding is not a char type");
		static_assert(TIsCharType<DestEncoding  >::Value, "Destination encoding is not a char type");

		// This is only defined for fixed-width encodings, because codepoints cannot be represented in a single variable-width code unit.
		static_assert(TIsFixedWidthCharEncoding_V<SourceEncoding>, "Source encoding is not fixed-width");

		if constexpr (TIsCharEncodingSimplyConvertibleTo_V<SourceEncoding, DestEncoding>)
		{
			// Simple conversions mean conversion is always possible
			return true;
		}
		else if constexpr (!TIsFixedWidthCharEncoding_V<DestEncoding>)
		{
			// Converting all codepoints to a variable-width encoding should always be possible
			return true;
		}
		else if constexpr (std::is_same_v<DestEncoding, ANSICHAR>)
		{
			return (uint32)Codepoint <= 0x7F;
		}
		else
		{
			// The logic above should hopefully mean this branch is only taken for UTF32CHAR->UCS2CHAR.
			// There's a variety of '16-bit' char types between platforms though, so let's just test sizes.
			static_assert(sizeof(SourceEncoding) == 4 && sizeof(DestEncoding) == 2, "Unimplemented conversion");

			// Can't encode more than 16-bit in UCS-2
			return (uint32)Codepoint <= 0xFFFF;
		}
	}


	/**
	 * Returns the string representing the name of the given encoding type.
	 *
	 * @return The name of the CharType as a TCHAR string.
	 */
	template <typename Encoding>
	static const TCHAR* GetEncodingTypeName();

	static const ANSICHAR* GetEncodingName()
	{
		return "UTF-16LE";
	}

	/**
	 * True if the encoding type of the string is some form of unicode
	 */
	static constexpr bool IsUnicodeEncoded = true;


	/**
	 * Converts the null-terminated Src string range from SourceEncoding to DestEncoding and writes it to the [Dest, Dest+DestSize) range, including a null terminator.
	 * If the Dest range is not big enough to hold the converted output, NULL is returned.  In this case, nothing should be assumed about the contents of Dest.
	 *
	 * @param Dest      The start of the destination buffer.
	 * @param DestSize  The size of the destination buffer.
	 * @param Src       The start of the string to convert.
	 * @return          A pointer to one past the last-written element.
	 */
	template <typename SourceEncoding, typename DestEncoding>
	static inline DestEncoding* Convert(DestEncoding* Dest, int32 DestSize, const SourceEncoding* Src)
	{
		if constexpr (TIsCharEncodingSimplyConvertibleTo_V<SourceEncoding, DestEncoding>)
		{
			for (;;)
			{
				if (DestSize == 0)
				{
					return nullptr;
				}

				if (!(*Dest++ = (DestEncoding)*Src++))
				{
					return Dest;
				}

				--DestSize;
			}
		}
		else if constexpr (TIsFixedWidthCharEncoding_V<SourceEncoding> && TIsFixedWidthCharEncoding_V<DestEncoding>)
		{
			DestEncoding*         DestCopy     = Dest;
			const SourceEncoding* SrcCopy      = Src;
			int32                 DestSizeCopy = DestSize;

			bool bInvalidChars = false;
			for (;;)
			{
				if (DestSize == 0)
				{
					Dest = nullptr;
					break;
				}

				SourceEncoding SrcCh = *Src++;
				*Dest++ = (DestEncoding)SrcCh;
				if (!SrcCh)
				{
					break;
				}
				bInvalidChars |= !CanConvertCodepoint<DestEncoding>(SrcCh);

				--DestSize;
			}

			if (bInvalidChars)
			{
				for (;;)
				{
					if (DestSizeCopy == 0)
					{
						break;
					}

					SourceEncoding SrcCh = *SrcCopy++;
					if (!SrcCh)
					{
						break;
					}
					if (!CanConvertCodepoint<DestEncoding>(SrcCh))
					{
						*DestCopy = UNICODE_BOGUS_CHAR_CODEPOINT;
					}
					++DestCopy;

					--DestSizeCopy;
				}
			}

			return Dest;
		}
		else
		{
			DestEncoding* Result = UE::Core::Private::Convert(Dest, DestSize, Src);
			if (Result)
			{
				*Result++ = (DestEncoding)0;
			}
			return Result;
		}
	}

	/**
	 * Converts the [Src, Src+SrcSize) string range from SourceEncoding to DestEncoding and writes it to the [Dest, Dest+DestSize) range.
	 * The Src range should contain a null terminator if a null terminator is required in the output.
	 * If the Dest range is not big enough to hold the converted output, NULL is returned.  In this case, nothing should be assumed about the contents of Dest.
	 *
	 * @param Dest      The start of the destination buffer.
	 * @param DestSize  The size of the destination buffer.
	 * @param Src       The start of the string to convert.
	 * @param SrcSize   The number of Src elements to convert.
	 * @return          A pointer to one past the last-written element.
	 */
	template <typename SourceEncoding, typename DestEncoding>
	static inline DestEncoding* Convert(DestEncoding* Dest, int32 DestSize, const SourceEncoding* Src, int32 SrcSize)
	{
		if constexpr (TIsCharEncodingCompatibleWith_V<SourceEncoding, DestEncoding>)
		{
			if (DestSize < SrcSize)
			{
				return nullptr;
			}

			return (DestEncoding*)Memcpy(Dest, Src, SrcSize * sizeof(SourceEncoding)) + SrcSize;
		}
		else if constexpr (TIsCharEncodingSimplyConvertibleTo_V<SourceEncoding, DestEncoding>)
		{
			const int32 Size = DestSize <= SrcSize ? DestSize : SrcSize;
			for (int I = 0; I < Size; ++I)
			{
				SourceEncoding SrcCh = Src[I];
				Dest[I] = (DestEncoding)SrcCh;
			}

			return DestSize < SrcSize ? nullptr : Dest + Size;
		}
		else if constexpr (TIsFixedWidthCharEncoding_V<SourceEncoding> && TIsFixedWidthCharEncoding_V<DestEncoding>)
		{
			const int32 Size = DestSize <= SrcSize ? DestSize : SrcSize;
			bool bInvalidChars = false;
			for (int I = 0; I < Size; ++I)
			{
				SourceEncoding SrcCh = Src[I];
				Dest[I] = (DestEncoding)SrcCh;
				bInvalidChars |= !CanConvertCodepoint<DestEncoding>(SrcCh);
			}

			if (bInvalidChars)
			{
				for (int I = 0; I < Size; ++I)
				{
					if (!CanConvertCodepoint<DestEncoding>(Src[I]))
					{
						Dest[I] = UNICODE_BOGUS_CHAR_CODEPOINT;
					}
				}
			}

			return DestSize < SrcSize ? nullptr : Dest + Size;
		}
		else
		{
			return UE::Core::Private::Convert(Dest, DestSize, Src, SrcSize);
		}
	}


	/**
	 * Returns the required buffer length for the null-terminated Src string when converted to the DestChar encoding.
	 * The returned length includes the space for the null terminator (equivalent to strlen+1).
	 * 
	 * @param  Src  The start of the string to convert.
	 * @return      The number of DestChar elements that Src will be converted into.
	 */
	template <typename DestEncoding, typename SourceEncoding>
	static int32 ConvertedLength(const SourceEncoding* Src)
	{
		if constexpr (TIsCharEncodingSimplyConvertibleTo_V<SourceEncoding, DestEncoding> || (TIsFixedWidthCharEncoding_V<SourceEncoding> && TIsFixedWidthCharEncoding_V<DestEncoding>))
		{
			int32 Result = 0;
			while (*Src)
			{
				++Src;
				++Result;
			}
			return Result + 1;
		}
		else
		{
			return UE::Core::Private::GetConvertedLength((DestEncoding*)nullptr, Src) + 1;
		}
	}

	/**
	 * Returns the required buffer length for the [Src, Src+SrcSize) string when converted to the DestChar encoding.
	 * The Src range should contain a null terminator if a null terminator is required in the output.
	 * 
	 * @param  Src     The start of the string to convert.
	 * @param  SrcSize The number of Src elements to convert.
	 * @return         The number of DestChar elements that Src will be converted into.
	 */
	template <typename DestEncoding, typename SourceEncoding>
	static int32 ConvertedLength(const SourceEncoding* Src, int32 SrcSize)
	{
		if constexpr (TIsCharEncodingSimplyConvertibleTo_V<SourceEncoding, DestEncoding> || (TIsFixedWidthCharEncoding_V<SourceEncoding> && TIsFixedWidthCharEncoding_V<DestEncoding>))
		{
			return SrcSize;
		}
		else
		{
			return UE::Core::Private::GetConvertedLength((DestEncoding*)nullptr, Src, SrcSize);
		}
	}

	CORE_API static int32 Strncmp(const ANSICHAR* String1, const ANSICHAR* String2, SIZE_T Count);
	CORE_API static int32 Strncmp(const WIDECHAR* String1, const ANSICHAR* String2, SIZE_T Count);
	CORE_API static int32 Strncmp(const UTF8CHAR* String1, const ANSICHAR* String2, SIZE_T Count);
	CORE_API static int32 Strncmp(const ANSICHAR* String1, const WIDECHAR* String2, SIZE_T Count);
	CORE_API static int32 Strncmp(const WIDECHAR* String1, const WIDECHAR* String2, SIZE_T Count);
	CORE_API static int32 Strncmp(const UTF8CHAR* String1, const WIDECHAR* String2, SIZE_T Count);
	CORE_API static int32 Strncmp(const ANSICHAR* String1, const UTF8CHAR* String2, SIZE_T Count);
	CORE_API static int32 Strncmp(const WIDECHAR* String1, const UTF8CHAR* String2, SIZE_T Count);
	CORE_API static int32 Strncmp(const UTF8CHAR* String1, const UTF8CHAR* String2, SIZE_T Count);

	/**
	 * Format to a destination buffer. Similar to vsnprintf.
	 *
	 * Writes the formatted output to Dest, up to a maximum of DestSize-1 characters, excluding the null terminator.
	 * Content of Dest is undefined when the return value is outside of the range [0, DestSize).
	 * A return value is calculated even when DestSize is 0, in which case Dest may be null.
	 * A null terminator is always written unless DestSize is 0.
	 * Success is indicated by ReturnValue >= 0 && ReturnValue < DestSize.
	 *
	 * %hs is always treated as UTF-8.
	 * %ls is always treated as UTF-16.
	 * %s is treated as UTF-8 for narrow format strings and UTF-16 for wide format strings.
	 *
	 * @param Dest   Pointer to the destination buffer to write to.
	 * @param DestSize   Maximum number of characters to write to Dest, including the null terminator.
	 * @param Format   Format string with which to format the output.
	 * @param Args   Arguments containing the data referenced by the format string.
	 * @return Number of characters, excluding the null terminator, that would be
	 *         written to Dest if DestSize was ignored, or -1 on error.
	 */
	[[nodiscard]] CORE_API static int32 FormatV(ANSICHAR* Dest, SIZE_T DestSize, const ANSICHAR* Format, va_list Args);
	[[nodiscard]] CORE_API static int32 FormatV(WIDECHAR* Dest, SIZE_T DestSize, const WIDECHAR* Format, va_list Args);
	[[nodiscard]] UE_REWRITE static int32 FormatV(UTF8CHAR* Dest, SIZE_T DestSize, const ANSICHAR* Format, va_list Args)
	{
		return FormatV((ANSICHAR*)Dest, DestSize, Format, Args);
	}
	[[nodiscard]] UE_REWRITE static int32 FormatV(UTF8CHAR* Dest, SIZE_T DestSize, const UTF8CHAR* Format, va_list Args)
	{
		return FormatV((ANSICHAR*)Dest, DestSize, (const ANSICHAR*)Format, Args);
	}

	UE_REWRITE static int32 GetVarArgs(ANSICHAR* Dest, SIZE_T DestSize, const ANSICHAR*& Fmt, va_list ArgPtr)
	{
		int32 Result = FormatV(Dest, DestSize, Fmt, ArgPtr);
		return (Result >= 0 && Result < (int32)DestSize) ? Result : -1;
	}

	UE_REWRITE static int32 GetVarArgs(WIDECHAR* Dest, SIZE_T DestSize, const WIDECHAR*& Fmt, va_list ArgPtr)
	{
		int32 Result = FormatV(Dest, DestSize, Fmt, ArgPtr);
		return (Result >= 0 && Result < (int32)DestSize) ? Result : -1;
	}

	UE_REWRITE static int32 GetVarArgs(UTF8CHAR* Dest, SIZE_T DestSize, const UTF8CHAR*& Fmt, va_list ArgPtr)
	{
		int32 Result = FormatV((ANSICHAR*)Dest, DestSize, (const ANSICHAR*)Fmt, ArgPtr);
		return (Result >= 0 && Result < (int32)DestSize) ? Result : -1;
	}

private:
	/**
	 * Forwarding function because we can't call FMemory::Memcpy directly due to #include ordering issues.
	 *
	 * @param Dest  The destination buffer.
	 * @param Src   The source buffer.
	 * @param Count The number of bytes to copy.
	 * @return      Dest
	 */
	static CORE_API void* Memcpy(void* Dest, const void* Src, SIZE_T Count);

protected:
#if DO_CHECK
	static CORE_API void ReportZeroStrncpySize();
#endif
};
