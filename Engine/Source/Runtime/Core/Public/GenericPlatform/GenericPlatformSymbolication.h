// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "Serialization/Archive.h"
#include "Templates/UnrealTypeTraits.h"

struct FProgramCounterSymbolInfo;
template <typename T> struct TCallTraits;

/**
 * File & line info for a debug symbol region
 */
struct FGenericPlatformSymbolInfo
{
	uint32 Line;
	uint64 Start;
	uint64 Length;
	int32 PathIdx;
	
	/**
	 * Serializes a symbol info from or into an archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param Info The info to serialize.
	 */
	friend FArchive& operator<<( FArchive& Ar, FGenericPlatformSymbolInfo& Info )
	{
		return Ar << Info.Line << Info.Start << Info.Length << Info.PathIdx;
	}
};


/**
 * Debug symbol information
 */
struct FGenericPlatformSymbolData
{
	uint64 Start;
	uint64 Length;
	int32 NameIdx;
	TArray<FGenericPlatformSymbolInfo> SymbolInfo;
	
	/**
	 * Serializes a symbol from or into an archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param Info The info to serialize.
	 */
	friend FArchive& operator<<( FArchive& Ar, FGenericPlatformSymbolData& Info )
	{
		return Ar << Info.Start << Info.Length << Info.NameIdx << Info.SymbolInfo;
	}
};


/**
 * Container for debug symbols corresponding to a single binary file
 */
struct FGenericPlatformSymbolDatabase
{
	FString Signature;
	FString Name;
	TArray<FGenericPlatformSymbolData> Symbols;
	TArray<FString> StringTable;
	
	/**
	 * Serializes a symbol container from or into an archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param Info The info to serialize.
	 */
	friend FArchive& operator<<( FArchive& Ar, FGenericPlatformSymbolDatabase& Info )
	{
		return Ar << Info.Signature << Info.Name << Info.Symbols << Info.StringTable;
	}
};


struct FGenericPlatformSymbolDatabaseKeyFuncs
{
	enum { bAllowDuplicateKeys = 0 };
	typedef TCallTraits<FString>::ParamType KeyInitType;
	typedef TCallTraits<FGenericPlatformSymbolDatabase>::ParamType ElementInitType;

	/**
	 * @return The key used to index the given element.
	 */
	static UE_FORCEINLINE_HINT KeyInitType GetSetKey(ElementInitType Element)
	{
		return Element.Signature;
	}

	/**
	 * @return True if the keys match.
	 */
	static UE_FORCEINLINE_HINT bool Matches(KeyInitType A,KeyInitType B)
	{
		return A == B;
	}

	/** Calculates a hash index for a key. */
	static UE_FORCEINLINE_HINT uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key);
	}
};


typedef TSet<FGenericPlatformSymbolDatabase, FGenericPlatformSymbolDatabaseKeyFuncs> FGenericPlatformSymbolDatabaseSet;


struct FGenericPlatformSymbolication
{
	/**
	 * @param Architecture Specify the architecture in case of universal binary, currently unused
	 */
	static CORE_API bool LoadSymbolDatabaseForBinary(FString SourceFolder, FString BinaryPath, FString BinarySignature, TOptional<FString> Architecture, FGenericPlatformSymbolDatabase& OutDatabase);
	static CORE_API bool SaveSymbolDatabaseForBinary(FString TargetFolder, FString Name, FGenericPlatformSymbolDatabase& Database);
	
	static CORE_API bool SymbolInfoForStrippedSymbol(FGenericPlatformSymbolDatabase const& Database, uint64 ProgramCounter, uint64 ModuleOffset, FString ModuleSignature, FProgramCounterSymbolInfo& Info);
	
	static bool SymbolInfoForAddress(uint64 ProgramCounter, FProgramCounterSymbolInfo& Info) { return false; }
};
