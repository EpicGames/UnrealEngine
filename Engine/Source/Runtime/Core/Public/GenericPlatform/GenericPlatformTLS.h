// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"


/**
 * It should be possible to provide a generic implementation as long as a threadID is provided. We don't do that yet.
 */
struct FGenericPlatformTLS
{
	static const uint32 InvalidTlsSlot = 0xFFFFFFFF;

	/**
	 * Return false if this is an invalid TLS slot
	 * @param SlotIndex the TLS index to check
	 * @return true if this looks like a valid slot
	 */
	static UE_FORCEINLINE_HINT bool IsValidTlsSlot(uint32 SlotIndex)
	{
		return SlotIndex != InvalidTlsSlot;
	}

#if 0 // provided for reference
	/**
	 * Returns the currently executing thread's id
	 */
	static UE_FORCEINLINE_HINT uint32 GetCurrentThreadId(void)
	{
	}

	/**
	 * Allocates a thread local store slot
	 */
	static UE_FORCEINLINE_HINT uint32 AllocTlsSlot(void)
	{
	}

	/**
	 * Sets a value in the specified TLS slot
	 *
	 * @param SlotIndex the TLS index to store it in
	 * @param Value the value to store in the slot
	 */
	static UE_FORCEINLINE_HINT void SetTlsValue(uint32 SlotIndex,void* Value)
	{
	}

	/**
	 * Reads the value stored at the specified TLS slot
	 *
	 * @return the value stored in the slot
	 */
	static UE_FORCEINLINE_HINT void* GetTlsValue(uint32 SlotIndex)
	{
	}

	/**
	 * Frees a previously allocated TLS slot
	 *
	 * @param SlotIndex the TLS index to store it in
	 */
	static UE_FORCEINLINE_HINT void FreeTlsSlot(uint32 SlotIndex)
	{
	}
#endif
};
