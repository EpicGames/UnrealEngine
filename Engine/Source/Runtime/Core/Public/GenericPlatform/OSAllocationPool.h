// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#if PLATFORM_HAS_FPlatformVirtualMemoryBlock
/*
 *  Perform checks that ensure that the pools are working as intended. This is not necessary in builds that are used for Shipping/Test or for the Development editor.
 */
#define UE4_TMEMORY_POOL_DO_SANITY_CHECKS		(UE_BUILD_DEBUG || (UE_BUILD_DEVELOPMENT && !UE_EDITOR))

/**
* Class for managing allocations of the size no larger than BlockSize.
*
* @param CommitAddressRange function to let the OS know that a range needs to be backed by physical RAM
* @param EvictAddressRange function to let the OS know that the RAM pages backing an address range can be evicted.
* @param RequiredAlignment alignment for the addresses returned from this pool
*/
template<
	SIZE_T RequiredAlignment
>
class TMemoryPool
{
protected:

	/** Size of a single block */
	SIZE_T		BlockSize;

	/** Beginning of the pool (an address in memory), cast to SIZE_T for arithmetic ops. */
	SIZE_T		AlignedPoolStart;

	/** End of the pool (an address in memory), cast to SIZE_T for arithmetic ops. */
	SIZE_T		AlignedPoolEnd;

	/** Num of the blocks to cache */
	SIZE_T		NumBlocks;

	/** A bit mask of the free blocks: 0 used, 1 free - because that way it's easier to scan */
	uint8*		Bitmask;

	/** Size of the bitmask in bytes */
	SIZE_T		BitmaskSizeInBytes;

	/** Current length of the stack. */
	SIZE_T		NumFreeBlocks;

	/** When we're allocating less than block size, only BlockSize - Size is going to be used. */
	SIZE_T 		UsefulMemorySize;

	FPlatformMemory::FPlatformVirtualMemoryBlock VMBlock;

#if UE4_TMEMORY_POOL_DO_SANITY_CHECKS
	FThreadSafeCounter NoConcurrentAccess; // Tests that this is not being accessed on multiple threads at once, see TestPAL mallocthreadtest
#endif // UE4_TMEMORY_POOL_DO_SANITY_CHECKS

public:

	TMemoryPool(SIZE_T InBlockSize, SIZE_T InAlignedPoolStart, SIZE_T InNumBlocks, uint8* InBitmask, FPlatformMemory::FPlatformVirtualMemoryBlock& InVMBlock)
		: BlockSize(InBlockSize)
		, AlignedPoolStart(InAlignedPoolStart)
		, AlignedPoolEnd(InAlignedPoolStart + InBlockSize * (SIZE_T)InNumBlocks)
		, NumBlocks(InNumBlocks)
		, Bitmask(InBitmask)
		, BitmaskSizeInBytes(BitmaskMemorySize(NumBlocks))
		, NumFreeBlocks(InNumBlocks)
		, UsefulMemorySize(0)
		, VMBlock(InVMBlock)
	{
#if UE4_TMEMORY_POOL_DO_SANITY_CHECKS
		checkf((AlignedPoolStart % RequiredAlignment) == 0, TEXT("Non-aligned pool address passed to a TMemoryPool"));
#endif // UE4_TMEMORY_POOL_DO_SANITY_CHECKS

		FMemory::Memset(Bitmask, 0xFF, BitmaskSizeInBytes);
		//checkf(NumFreeBlocks == CalculateFreeBlocksInBitmap(), TEXT("Mismatch between a bitmap and NumFreeBlocks at the very beginning"));

		// decommit all the memory
		VMBlock.DecommitByPtr(reinterpret_cast<void *>(AlignedPoolStart), Align(NumBlocks * BlockSize, FPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment()));
	}

	/** We always allocate in BlockSize chunks, Size is only passed for more accurate Commit() */
	void* Allocate(SIZE_T Size)
	{
#if UE4_TMEMORY_POOL_DO_SANITY_CHECKS
		checkf(Size <= BlockSize, TEXT("Attempting to allocate %llu bytes from a memory pool of %llu byte blocks"), (uint64)Size, (uint64)BlockSize);

		checkf(NoConcurrentAccess.Increment() == 1, TEXT("TMemoryPool is being accessed on multiple threads. The class is not thread safe, add locking!."));
#endif // UE4_TMEMORY_POOL_DO_SANITY_CHECKS

		void* Address = nullptr;
		if (LIKELY(NumFreeBlocks > 0))
		{
			//checkf(NumFreeBlocks == CalculateFreeBlocksInBitmap(), TEXT("Mismatch between a bitmap and NumFreeBlocks before Allocate()ing"));
			Address = FindFirstFreeAndMarkUsed();
			--NumFreeBlocks;
			UsefulMemorySize += Size;
			//checkf(NumFreeBlocks == CalculateFreeBlocksInBitmap(), TEXT("Mismatch between a bitmap and NumFreeBlocks after Allocate()ing"));

#if UE4_TMEMORY_POOL_DO_SANITY_CHECKS
			checkf(Address != nullptr, TEXT("NumFreeBlocks and bitmask of the free blocks are not in sync - bug in TMemoryPool"));
#endif // UE4_TMEMORY_POOL_DO_SANITY_CHECKS

			// make sure this address range is backed by the actual RAM
			VMBlock.CommitByPtr(Address, Align(Size, FPlatformMemory::FPlatformVirtualMemoryBlock::GetCommitAlignment()));

		}

#if UE4_TMEMORY_POOL_DO_SANITY_CHECKS
		checkf(NoConcurrentAccess.Decrement() == 0, TEXT("TMemoryPool is being accessed on multiple threads. The class is not thread safe, add locking!."));
#endif // UE4_TMEMORY_POOL_DO_SANITY_CHECKS

		return Address;
	}

	/** We always free BlockSize-d chunks. */
	void Free(void *Ptr, SIZE_T Size)
	{
		// first, check if the block is ours at all. This may happen if allocations spilled over to regular BAFO() when the pool is full, but it is unlikely.
#if UE4_TMEMORY_POOL_DO_SANITY_CHECKS
		checkf(WasAllocatedFromThisPool(Ptr, BlockSize), TEXT("Address passed to Free() of a pool of block size %llu was not allocated in it (address: %p, boundaries: %p - %p"),
			(uint64)BlockSize,
			Ptr,
			reinterpret_cast<void *>(AlignedPoolStart),
			reinterpret_cast<void *>(AlignedPoolEnd)
			);

		checkf((reinterpret_cast<SIZE_T>(Ptr) % RequiredAlignment == 0), TEXT("Address passed to Free() of a pool of block size %llu was not aligned to %llu bytes (address: %p)"),
			(uint64)BlockSize,
			(uint64)RequiredAlignment,
			Ptr
			);

		checkf(NoConcurrentAccess.Increment() == 1, TEXT("TMemoryPool is being accessed on multiple threads. The class is not thread safe, add locking!."));
#endif // UE4_TMEMORY_POOL_DO_SANITY_CHECKS

		// add the pointer back to the pool
		//checkf(NumFreeBlocks == CalculateFreeBlocksInBitmap(), TEXT("Mismatch between a bitmap and NumFreeBlocks before Free()ing"));
		MarkFree(Ptr);
		NumFreeBlocks++;
		UsefulMemorySize -= Size;
		//checkf(NumFreeBlocks == CalculateFreeBlocksInBitmap(), TEXT("Mismatch between a bitmap and NumFreeBlocks after Free()ing"));

		// evict this memory
		VMBlock.DecommitByPtr(Ptr, BlockSize);

#if UE4_TMEMORY_POOL_DO_SANITY_CHECKS
		// check that we never free more than we allocated
		checkf(NumFreeBlocks <= NumBlocks, TEXT("Too many frees!"));

		checkf(NoConcurrentAccess.Decrement() == 0, TEXT("TMemoryPool is being accessed on multiple threads. The class is not thread safe, add locking!."));
#endif // UE4_TMEMORY_POOL_DO_SANITY_CHECKS
	}

	static SIZE_T BitmaskMemorySize(SIZE_T NumBlocks)
	{
		return (NumBlocks / 8) + ((NumBlocks & 7) ? 1 : 0); 
	}

	void MarkFree(void *Ptr)
	{
		// calculate the bit number
		SIZE_T PtrOffset = reinterpret_cast<SIZE_T>(Ptr);
		SIZE_T BitIndex = (PtrOffset - AlignedPoolStart) / BlockSize;

#if UE4_TMEMORY_POOL_DO_SANITY_CHECKS
		checkf(BitIndex < NumBlocks, TEXT("Incorrect pointer %p passed to MarkFree()"), Ptr);
#endif // UE4_TMEMORY_POOL_DO_SANITY_CHECKS

		SIZE_T ByteIndex = BitIndex / 8;
		uint8 Byte = Bitmask[ByteIndex];

		int32 IndexInByte = static_cast<int32>(BitIndex & 0x7);
#if UE4_TMEMORY_POOL_DO_SANITY_CHECKS
		checkf((Byte & (1 << IndexInByte)) == 0, TEXT("MarkFree() - double freeing the pointer %p"), Ptr);
#endif // UE4_TMEMORY_POOL_DO_SANITY_CHECKS
		Byte |= (1 << IndexInByte);
		Bitmask[ByteIndex] = Byte;
	}

	void* FindFirstFreeAndMarkUsed()
	{
		uint8* CurPtr = Bitmask;

		SIZE_T BitmaskSizeInQWords = BitmaskSizeInBytes / 8;
		SIZE_T IdxQword = 0;
		while(IdxQword < BitmaskSizeInQWords)
		{
			uint64 Qword = *reinterpret_cast<uint64*>(CurPtr);
			if (UNLIKELY(Qword != 0))
			{
				// find the first free in it
				// the memory is assumed to be little endian, so we count trailing
				SIZE_T IndexOfFirstFreeInQword = FMath::CountTrailingZeros64(Qword);
				
				// mark as used
				Qword &= ~(1ULL << IndexOfFirstFreeInQword);
				*reinterpret_cast<uint64*>(CurPtr) = Qword;

				// return the address
				SIZE_T IndexOfFirstBlock = static_cast<SIZE_T>(CurPtr - Bitmask) * 8 + IndexOfFirstFreeInQword;
#if UE4_TMEMORY_POOL_DO_SANITY_CHECKS
				checkf(IndexOfFirstBlock < NumBlocks, TEXT("Allocating outside of pool - TMemoryPool error."));
#endif // UE4_TMEMORY_POOL_DO_SANITY_CHECKS
				return reinterpret_cast<void *>(AlignedPoolStart + IndexOfFirstBlock * BlockSize);
			}

			CurPtr += 8;
			++IdxQword;
		}

		// search byte to byte
		SIZE_T MaxIndexInLastByte = (NumBlocks & 0x7) ? (NumBlocks & 0x7) : 8;
	
		uint8* BitmaskEnd = Bitmask + BitmaskSizeInBytes;
		while(CurPtr < BitmaskEnd)
		{
			uint8 Byte = *CurPtr;
			if (UNLIKELY(Byte != 0))
			{
				// find the first free
				SIZE_T IndexOfFirstFreeInByte = FMath::CountTrailingZeros(static_cast<int32>(Byte));

				// the last byte needs a special attention
				if (LIKELY(CurPtr < BitmaskEnd - 1 || IndexOfFirstFreeInByte < MaxIndexInLastByte))
				{
					// mark as used
					Byte &= ~(1 << IndexOfFirstFreeInByte);
					*CurPtr = Byte;

					// return the address
					SIZE_T IndexOfFirstBlock = static_cast<SIZE_T>(CurPtr - Bitmask) * 8 + IndexOfFirstFreeInByte;
#if UE4_TMEMORY_POOL_DO_SANITY_CHECKS
					checkf(IndexOfFirstBlock < NumBlocks, TEXT("Allocating outside of pool - TMemoryPool error."));
#endif // UE4_TMEMORY_POOL_DO_SANITY_CHECKS
					return reinterpret_cast<void *>(AlignedPoolStart + IndexOfFirstBlock * BlockSize);
				}
			}

			++CurPtr;
		}

		return nullptr;
	}

	/** Debugging function */
	SIZE_T CalculateFreeBlocksInBitmap()
	{
		SIZE_T NumFree = 0;

		uint8* CurPtr = Bitmask;
		uint8* BitmaskEnd = Bitmask + BitmaskSizeInBytes;
		while(CurPtr < BitmaskEnd - sizeof(uint64))
		{
			NumFree += FMath::CountBits64(*reinterpret_cast<uint64*>(CurPtr));
			CurPtr += sizeof(uint64);
		}

		while(CurPtr < BitmaskEnd - 1)
		{
			NumFree += FMath::CountBits(*CurPtr);
			CurPtr ++;
		}

		// handle last byte manually
		if ((NumBlocks & 0x7) == 0)
		{
			NumFree += FMath::CountBits(*CurPtr);
		}
		else
		{
			uint8 LastByte = *CurPtr;

			SIZE_T MaxIndexInLastByte = (NumBlocks & 0x7);
			for(SIZE_T Idx = 0; Idx < MaxIndexInLastByte; ++Idx)
			{
				NumFree += (LastByte & (1 << Idx)) ? 1 : 0;
			}
		}

		return NumFree;
	}

	/** Returns true if we can allocate this much memory from this pool. */
	bool CanAllocateFromThisPool(SIZE_T Size)
	{
		return BlockSize >= Size;
	}

	/** Returns true if this allocation came from this pool. */
	bool WasAllocatedFromThisPool(void* Ptr, SIZE_T Size)
	{
		// this extra size check is largely redundant and this function is on a hot path
		return /*BlockSize >= Size &&*/ reinterpret_cast<SIZE_T>(Ptr) >= AlignedPoolStart && reinterpret_cast<SIZE_T>(Ptr) < AlignedPoolEnd;
	}

	bool IsEmpty() const
	{
		return NumFreeBlocks == NumBlocks;
	}

	/** Returns memory size that we can actually allocate from the pool (mostly for malloc stats) */
	uint64 GetAllocatableMemorySize() const
	{
		return NumFreeBlocks * BlockSize;
	}

	/** Returns overhead caused by allocating less than BlockSize (mostly for malloc stats) */
	uint64 GetOverheadSize() const
	{
		return (NumBlocks - NumFreeBlocks) * BlockSize - UsefulMemorySize;
	}

	void PrintDebugInfo()
	{
		printf("BlockSize: %llu NumAllocated/TotalBlocks = %llu/%llu\n", (uint64)BlockSize, (uint64)(NumBlocks - NumFreeBlocks), (uint64)NumBlocks);
	}
};

#endif