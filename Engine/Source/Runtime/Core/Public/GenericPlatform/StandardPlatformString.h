// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformStricmp.h"
#include "GenericPlatform/GenericPlatformString.h"
#include "Misc/Char.h"

// IWYU pragma: begin_exports
#include <stdio.h>
#include <stdlib.h>
#include <string.h>
#include <wchar.h>
// IWYU pragma: end_exports

#if PLATFORM_USE_GENERIC_STRING_IMPLEMENTATION && !PLATFORM_TCHAR_IS_CHAR16

/**
* Standard implementation
**/
struct FStandardPlatformString : public FGenericPlatformString
{
	using FGenericPlatformString::Stricmp;
	using FGenericPlatformString::Strncmp;
	using FGenericPlatformString::Strnicmp;

	template <typename CharType>
	static inline CharType* Strupr(CharType* Dest, SIZE_T DestCount)
	{
		for (CharType* Char = Dest; *Char && DestCount > 0; Char++, DestCount--)
		{
			*Char = TChar<CharType>::ToUpper(*Char);
		}
		return Dest;
	}

public:

	/**
	 * Unicode implementation
	 **/
	static inline WIDECHAR* Strcpy(WIDECHAR* Dest, const WIDECHAR* Src)
	{
// Skip suggestions about using wcscpy_s instead.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return wcscpy(Dest, Src);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// This version was deprecated because Strcpy taking a size field does not match the CRT strcpy, and on some platforms the size field was being ignored.
	// If the memzeroing causes a performance problem, request a new function StrcpyTruncate.
	UE_DEPRECATED(5.6, "Use Strncpy instead. Note that Strncpy has a behavior difference from Strcpy: it memzeroes the entire DestCount-sized buffer after the end of string.")
	static inline WIDECHAR* Strcpy(WIDECHAR* Dest, SIZE_T DestCount, const WIDECHAR* Src)
	{
		// Skip suggestions about using wcscpy_s instead.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return wcscpy(Dest, Src);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static inline WIDECHAR* Strncpy(WIDECHAR* Dest, const WIDECHAR* Src, SIZE_T MaxLen)
	{
#if DO_CHECK
		if (UNLIKELY(MaxLen == 0))
		{
			ReportZeroStrncpySize();
		}
#endif

// Skip suggestions about using wcsncpy_s instead.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		wcsncpy(Dest, Src, MaxLen-1);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Dest[MaxLen-1]=0;
		return Dest;
	}

	static inline WIDECHAR* Strcat(WIDECHAR* Dest, const WIDECHAR* Src)
	{
// Skip suggestions about using wcscat_s instead.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return (WIDECHAR*)wcscat( Dest, Src );
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// This version was deprecated because Strcat taking a size field does not match the CRT strcat, and on some platforms the size field was being ignored.
	UE_DEPRECATED(5.6, "Use Strncat instead. !!NOTE THAT STRNCAT takes SrcLen rather than DestCount. You must call Strncat(Dest, Src, DestCount - Strlen(Dest) - 1).")
	static inline WIDECHAR* Strcat(WIDECHAR* Dest, SIZE_T DestCount, const WIDECHAR* Src)
	{
		// Skip suggestions about using wcscat_s instead.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return (WIDECHAR*)wcscat(Dest, Src);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static inline WIDECHAR* Strncat(WIDECHAR* Dest, const WIDECHAR* Src, SIZE_T SrcLen)
	{
		if (!SrcLen || !*Src)
		{
			return Dest;
		}
		WIDECHAR* NewDest = Dest + Strlen(Dest);
		SIZE_T AppendedCount = 0;
		do
		{
			*NewDest++ = *Src++;
			++AppendedCount;
		} while (AppendedCount < SrcLen && *Src);
		*NewDest = 0;
		return Dest;
	}

	static UE_FORCEINLINE_HINT int32 Strcmp( const WIDECHAR* String1, const WIDECHAR* String2 )
	{
		return wcscmp(String1, String2);
	}

	static UE_FORCEINLINE_HINT int32 Strncmp( const WIDECHAR* String1, const WIDECHAR* String2, SIZE_T Count )
	{
		return wcsncmp( String1, String2, Count );
	}

	static UE_FORCEINLINE_HINT int32 Strlen( const WIDECHAR* String )
	{
		return (int32)wcslen( String );
	}

	static UE_FORCEINLINE_HINT int32 Strnlen( const WIDECHAR* String, SIZE_T StringSize )
	{
		return (int32)wcsnlen_s( String, StringSize );
	}

	static UE_FORCEINLINE_HINT const WIDECHAR* Strstr( const WIDECHAR* String, const WIDECHAR* Find)
	{
		return wcsstr( String, Find );
	}

	static UE_FORCEINLINE_HINT const WIDECHAR* Strchr( const WIDECHAR* String, WIDECHAR C)
	{
		return wcschr( String, C );
	}

	static UE_FORCEINLINE_HINT const WIDECHAR* Strrchr( const WIDECHAR* String, WIDECHAR C)
	{
		return wcsrchr( String, C );
	}

	static UE_FORCEINLINE_HINT int32 Atoi(const WIDECHAR* String)
	{
		return (int32)wcstol( String, NULL, 10 );
	}

	static UE_FORCEINLINE_HINT int64 Atoi64(const WIDECHAR* String)
	{
		return wcstoll( String, NULL, 10 );
	}

	static UE_FORCEINLINE_HINT float Atof(const WIDECHAR* String)
	{
		return wcstof( String, NULL );
	}

	static UE_FORCEINLINE_HINT double Atod(const WIDECHAR* String)
	{
		return wcstod( String, NULL );
	}

	static UE_FORCEINLINE_HINT int32 Strtoi( const WIDECHAR* Start, WIDECHAR** End, int32 Base )
	{
		return (int32)wcstol( Start, End, Base );
	}

	static UE_FORCEINLINE_HINT int64 Strtoi64( const WIDECHAR* Start, WIDECHAR** End, int32 Base )
	{
		return wcstoll( Start, End, Base );
	}

	static UE_FORCEINLINE_HINT uint64 Strtoui64( const WIDECHAR* Start, WIDECHAR** End, int32 Base )
	{
		return wcstoull( Start, End, Base );
	}

	static inline WIDECHAR* Strtok(WIDECHAR* StrToken, const WIDECHAR* Delim, WIDECHAR** Context)
	{
// Skip suggestions about using wcstok_s instead.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return wcstok(StrToken, Delim, Context);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Ansi implementation
	 **/
	static inline ANSICHAR* Strcpy(ANSICHAR* Dest, const ANSICHAR* Src)
	{
// Skip suggestions about using strcpy_s instead.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return strcpy( Dest, Src );
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// This version was deprecated because Strcpy taking a size field does not match the CRT strcpy, and on some platforms the size field was being ignored.
	// If the memzeroing causes a performance problem, request a new function StrcpyTruncate.
	UE_DEPRECATED(5.6, "Use Strncpy instead. Note that Strncpy has a behavior difference from Strcpy: it memzeroes the entire DestCount-sized buffer after the end of string.")
	static inline ANSICHAR* Strcpy(ANSICHAR* Dest, SIZE_T DestCount, const ANSICHAR* Src)
	{
		// Skip suggestions about using strcpy_s instead.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return strcpy(Dest, Src);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static inline ANSICHAR* Strncpy(ANSICHAR* Dest, const ANSICHAR* Src, SIZE_T MaxLen)
	{
#if DO_CHECK
		if (UNLIKELY(MaxLen == 0))
		{
			ReportZeroStrncpySize();
		}
#endif

// Skip suggestions about using strncpy_s instead.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		::strncpy(Dest, Src, MaxLen);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Dest[MaxLen-1]=0;
		return Dest;
	}

	static inline ANSICHAR* Strcat(ANSICHAR* Dest, const ANSICHAR* Src)
	{
// Skip suggestions about using strcat_s instead.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return strcat( Dest, Src );
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// This version was deprecated because Strcat taking a size field does not match the CRT strcat, and on some platforms the size field was being ignored.
	UE_DEPRECATED(5.6, "Use Strncat instead. !!NOTE THAT STRNCAT takes SrcLen rather than DestSize. You must call Strncat(Dest, Src, DestCount - Strlen(Dest) - 1).")
	static inline ANSICHAR* Strcat(ANSICHAR* Dest, SIZE_T DestCount, const ANSICHAR* Src)
	{
// Skip suggestions about using strcat_s instead.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return strcat(Dest, Src);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static UE_FORCEINLINE_HINT ANSICHAR* Strncat(ANSICHAR* Dest, const ANSICHAR* Src, SIZE_T SrcLen)
	{
		return (ANSICHAR*)strncat(Dest, Src, SrcLen);
	}

	static UE_FORCEINLINE_HINT int32 Strcmp( const ANSICHAR* String1, const ANSICHAR* String2 )
	{
		return strcmp(String1, String2);
	}

	static UE_FORCEINLINE_HINT int32 Strncmp( const ANSICHAR* String1, const ANSICHAR* String2, SIZE_T Count )
	{
		return strncmp( String1, String2, Count );
	}

	static UE_FORCEINLINE_HINT int32 Strlen( const ANSICHAR* String )
	{
		return (int32)strlen( String );
	}

	static UE_FORCEINLINE_HINT int32 Strnlen( const ANSICHAR* String, SIZE_T StringSize )
	{
		return (int32)strnlen_s( String, StringSize );
	}

	static UE_FORCEINLINE_HINT const ANSICHAR* Strstr( const ANSICHAR* String, const ANSICHAR* Find)
	{
		return strstr(String, Find);
	}

	static UE_FORCEINLINE_HINT const ANSICHAR* Strchr( const ANSICHAR* String, ANSICHAR C)
	{
		return strchr(String, C);
	}

	static UE_FORCEINLINE_HINT const ANSICHAR* Strrchr( const ANSICHAR* String, ANSICHAR C)
	{
		return strrchr(String, C);
	}

	static UE_FORCEINLINE_HINT int32 Atoi(const ANSICHAR* String)
	{
		return atoi( String );
	}

	static UE_FORCEINLINE_HINT int64 Atoi64(const ANSICHAR* String)
	{
		return strtoll( String, NULL, 10 );
	}

	static UE_FORCEINLINE_HINT float Atof(const ANSICHAR* String)
	{
		return (float)atof( String );
	}

	static UE_FORCEINLINE_HINT double Atod(const ANSICHAR* String)
	{
		return atof( String );
	}

	static UE_FORCEINLINE_HINT int32 Strtoi( const ANSICHAR* Start, ANSICHAR** End, int32 Base )
	{
		return (int32)strtol( Start, End, Base );
	}

	static UE_FORCEINLINE_HINT int64 Strtoi64( const ANSICHAR* Start, ANSICHAR** End, int32 Base )
	{
		return strtoll(Start, End, Base);
	}

	static UE_FORCEINLINE_HINT uint64 Strtoui64( const ANSICHAR* Start, ANSICHAR** End, int32 Base )
	{
		return strtoull(Start, End, Base);
	}

	static inline ANSICHAR* Strtok(ANSICHAR* StrToken, const ANSICHAR* Delim, ANSICHAR** Context)
	{
// Skip suggestions about using strtok_s instead.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return strtok(StrToken, Delim);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * UTF8CHAR implementation.
	 */
	static UE_FORCEINLINE_HINT UTF8CHAR* Strcpy(UTF8CHAR* Dest, const UTF8CHAR* Src)
	{
		return (UTF8CHAR*)Strcpy((ANSICHAR*)Dest, (const ANSICHAR*)Src);
	}

	// This version was deprecated because Strcpy taking a size field does not match the CRT strcpy, and on some platforms the size field was being ignored.
	// If the memzeroing causes a performance problem, request a new function StrcpyTruncate.
	UE_DEPRECATED(5.6, "Use Strncpy instead. Note that Strncpy has a behavior difference from Strcpy: it memzeroes the entire DestCount-sized buffer after the end of string.")
	static inline UTF8CHAR* Strcpy(UTF8CHAR* Dest, SIZE_T DestCount, const UTF8CHAR* Src)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return (UTF8CHAR*)Strcpy((ANSICHAR*)Dest, DestCount, (const ANSICHAR*)Src);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static UE_FORCEINLINE_HINT UTF8CHAR* Strncpy(UTF8CHAR* Dest, const UTF8CHAR* Src, SIZE_T MaxLen)
	{
		return (UTF8CHAR*)Strncpy((ANSICHAR*)Dest, (const ANSICHAR*)Src, MaxLen);
	}

	static UE_FORCEINLINE_HINT UTF8CHAR* Strcat(UTF8CHAR* Dest, const UTF8CHAR* Src)
	{
		return (UTF8CHAR*)Strcat((ANSICHAR*)Dest, (const ANSICHAR*)Src);
	}

	// This version was deprecated because Strcat taking a size field does not match the CRT strcat, and on some platforms the size field was being ignored.
	UE_DEPRECATED(5.6, "Use Strncat instead. !!NOTE THAT STRNCAT takes SrcLen rather than DestSize. You must call Strncat(Dest, Src, DestCount - Strlen(Dest) - 1).")
	static inline UTF8CHAR* Strcat(UTF8CHAR* Dest, SIZE_T DestCount, const UTF8CHAR* Src)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return (UTF8CHAR*)Strcat((ANSICHAR*)Dest, DestCount, (const ANSICHAR*)Src);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static UE_FORCEINLINE_HINT UTF8CHAR* Strncat(UTF8CHAR* Dest, const UTF8CHAR* Src, SIZE_T SrcLen)
	{
		return (UTF8CHAR*)Strncat((ANSICHAR*)Dest, (const ANSICHAR*)Src, SrcLen);
	}

	static UE_FORCEINLINE_HINT int32 Strcmp(const UTF8CHAR* String1, const UTF8CHAR* String2)
	{
		return Strcmp((const ANSICHAR*)String1, (const ANSICHAR*)String2);
	}

	static UE_FORCEINLINE_HINT int32 Strncmp(const UTF8CHAR* String1, const UTF8CHAR* String2, SIZE_T Count)
	{
		return Strncmp((const ANSICHAR*)String1, (const ANSICHAR*)String2, Count);
	}

	static UE_FORCEINLINE_HINT int32 Strlen(const UTF8CHAR* String)
	{
		return Strlen((const ANSICHAR*)String);
	}

	static UE_FORCEINLINE_HINT int32 Strnlen(const UTF8CHAR* String, SIZE_T StringSize)
	{
		return Strnlen((const ANSICHAR*)String, StringSize);
	}

	static UE_FORCEINLINE_HINT const UTF8CHAR* Strstr(const UTF8CHAR* String, const UTF8CHAR* Find)
	{
		return (const UTF8CHAR*)Strstr((const ANSICHAR*)String, (const ANSICHAR*)Find);
	}

	static UE_FORCEINLINE_HINT const UTF8CHAR* Strchr(const UTF8CHAR* String, UTF8CHAR C)
	{
		return (const UTF8CHAR*)Strchr((const ANSICHAR*)String, (ANSICHAR)C);
	}

	static UE_FORCEINLINE_HINT const UTF8CHAR* Strrchr(const UTF8CHAR* String, UTF8CHAR C)
	{
		return (const UTF8CHAR*)Strrchr((const ANSICHAR*)String, (ANSICHAR)C);
	}

	static UE_FORCEINLINE_HINT int32 Atoi(const UTF8CHAR* String)
	{
		return Atoi((const ANSICHAR*)String);
	}

	static UE_FORCEINLINE_HINT int64 Atoi64(const UTF8CHAR* String)
	{
		return Atoi64((const ANSICHAR*)String);
	}

	static UE_FORCEINLINE_HINT float Atof(const UTF8CHAR* String)
	{
		return Atof((const ANSICHAR*)String);
	}

	static UE_FORCEINLINE_HINT double Atod(const UTF8CHAR* String)
	{
		return Atod((const ANSICHAR*)String);
	}

	static UE_FORCEINLINE_HINT int32 Strtoi(const UTF8CHAR* Start, UTF8CHAR** End, int32 Base)
	{
		return Strtoi((const ANSICHAR*)Start, (ANSICHAR**)End, Base);
	}

	static UE_FORCEINLINE_HINT int64 Strtoi64(const UTF8CHAR* Start, UTF8CHAR** End, int32 Base)
	{
		return Strtoi64((const ANSICHAR*)Start, (ANSICHAR**)End, Base);
	}

	static UE_FORCEINLINE_HINT uint64 Strtoui64(const UTF8CHAR* Start, UTF8CHAR** End, int32 Base)
	{
		return Strtoui64((const ANSICHAR*)Start, (ANSICHAR**)End, Base);
	}

	static UE_FORCEINLINE_HINT UTF8CHAR* Strtok(UTF8CHAR* StrToken, const UTF8CHAR* Delim, UTF8CHAR** Context)
	{
		return (UTF8CHAR*)Strtok((ANSICHAR*)StrToken, (const ANSICHAR*)Delim, (ANSICHAR**)Context);
	}

	/**
	 * UCS2 implementation
	 **/

	static inline int32 Strlen( const UCS2CHAR* String )
	{
		int32 Result = 0;
		while (*String++)
		{
			++Result;
		}

		return Result;
	}

	static inline int32 Strnlen( const UCS2CHAR* String, SIZE_T StringSize )
	{
		int32 Result = 0;
		while (StringSize-- > 0 && *String++)
		{
			++Result;
		}

		return Result;
	}
};

#endif
