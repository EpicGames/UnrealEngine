// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/OutputDevice.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformMutex.h"

struct FCachedOSPageAllocator
{
protected:
	struct FFreePageBlock
	{
		void*  Ptr;
		SIZE_T ByteSize;

		FFreePageBlock() 
		{
			Ptr      = nullptr;
			ByteSize = 0;
		}
	};

	void* AllocateImpl(SIZE_T Size, uint32 CachedByteLimit, FFreePageBlock* First, FFreePageBlock* Last, uint32& FreedPageBlocksNum, SIZE_T& CachedTotal, UE::FPlatformRecursiveMutex* Mutex);
	void FreeImpl(void* Ptr, SIZE_T Size, uint32 NumCacheBlocks, uint32 CachedByteLimit, FFreePageBlock* First, uint32& FreedPageBlocksNum, SIZE_T& CachedTotal, UE::FPlatformRecursiveMutex* Mutex, bool ThreadIsTimeCritical);
	void FreeAllImpl(FFreePageBlock* First, uint32& FreedPageBlocksNum, SIZE_T& CachedTotal, UE::FPlatformRecursiveMutex* Mutex);

	static bool IsOSAllocation(SIZE_T Size, uint32 CachedByteLimit)
	{
		return (FPlatformMemory::BinnedPlatformHasMemoryPoolForThisSize(Size) || Size > CachedByteLimit / 4);
	}
};

template <uint32 NumCacheBlocks, uint32 CachedByteLimit>
struct TCachedOSPageAllocator : private FCachedOSPageAllocator
{
	TCachedOSPageAllocator()
		: CachedTotal(0)
		, FreedPageBlocksNum(0)
	{
	}

	UE_FORCEINLINE_HINT void* Allocate(SIZE_T Size, uint32 AllocationHint = 0, UE::FPlatformRecursiveMutex* Mutex = nullptr)
	{
		return AllocateImpl(Size, CachedByteLimit, FreedPageBlocks, FreedPageBlocks + FreedPageBlocksNum, FreedPageBlocksNum, CachedTotal, Mutex);
	}

	void Free(void* Ptr, SIZE_T Size, UE::FPlatformRecursiveMutex* Mutex = nullptr, bool ThreadIsTimeCritical = false)
	{
		return FreeImpl(Ptr, Size, ThreadIsTimeCritical ? NumCacheBlocks*2 : NumCacheBlocks, CachedByteLimit, FreedPageBlocks, FreedPageBlocksNum, CachedTotal, Mutex, ThreadIsTimeCritical);
	}

	void FreeAll(UE::FPlatformRecursiveMutex* Mutex = nullptr)
	{
		return FreeAllImpl(FreedPageBlocks, FreedPageBlocksNum, CachedTotal, Mutex);
	}

	// Refresh cached os allocator if needed. Does nothing for this implementation
	void Refresh()
	{
	}

	void UpdateStats()
	{
	}

	uint64 GetCachedFreeTotal() const
	{
		return CachedTotal;
	}

	uint64 GetCachedImmediatelyFreeable() const
	{
		return GetCachedFreeTotal();
	}

	bool IsOSAllocation(SIZE_T Size)
	{
		return FCachedOSPageAllocator::IsOSAllocation(Size, CachedByteLimit);
	}

	void DumpAllocatorStats(class FOutputDevice& Ar)
	{
		Ar.Logf(TEXT("CachedOSPageAllocator = %fkb"), (double)GetCachedFreeTotal() / 1024.0);
	}

private:
	FFreePageBlock FreedPageBlocks[NumCacheBlocks*2];
	SIZE_T         CachedTotal;
	uint32         FreedPageBlocksNum;
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "HAL/CriticalSection.h"
#endif
