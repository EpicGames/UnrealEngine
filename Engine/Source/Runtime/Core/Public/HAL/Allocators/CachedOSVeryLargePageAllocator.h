// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Templates/UnrealTemplate.h"
#include "Containers/List.h"
#include "CoreTypes.h"
#include "HAL/Allocators/CachedOSPageAllocator.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformMutex.h"
#include "HAL/UnrealMemory.h"


#if UE_USE_VERYLARGEPAGEALLOCATOR

#define CACHEDOSVERYLARGEPAGEALLOCATOR_BYTE_LIMIT (128*1024*1024)
//#define CACHEDOSVERYLARGEPAGEALLOCATOR_MAX_CACHED_OS_FREES (CACHEDOSVERYLARGEPAGEALLOCATOR_BYTE_LIMIT/(1024*64))
#define CACHEDOSVERYLARGEPAGEALLOCATOR_MAX_CACHED_OS_FREES (256)

#ifndef UE_VERYLARGEPAGEALLOCATOR_RESERVED_SIZE_IN_GB
#define UE_VERYLARGEPAGEALLOCATOR_RESERVED_SIZE_IN_GB 4	//default to 4GB
#endif

#ifndef UE_VERYLARGEPAGEALLOCATOR_PAGESIZE_KB
#define UE_VERYLARGEPAGEALLOCATOR_PAGESIZE_KB 2048	//default to 2MB
#endif

class FCachedOSVeryLargePageAllocator
{
	static constexpr uint64 AddressSpaceToReserve = ((1024LL * 1024LL * 1024LL) * UE_VERYLARGEPAGEALLOCATOR_RESERVED_SIZE_IN_GB);
	static constexpr uint64 AddressSpaceToReserveForSmallPool = AddressSpaceToReserve / 2;

	static constexpr uint64 SizeOfLargePage = (UE_VERYLARGEPAGEALLOCATOR_PAGESIZE_KB * 1024);
	static constexpr uint64 SizeOfSubPage = (1024 * 64);
	static constexpr uint64 NumberOfLargePages = (AddressSpaceToReserve / SizeOfLargePage);
	static constexpr uint64 NumberOfSubPagesPerLargePage = (SizeOfLargePage / SizeOfSubPage);
public:

	FCachedOSVeryLargePageAllocator()
		: bEnabled(true)
		, CachedFree(0)
		, ImmediatelyFreeable(0)
	{
		Init();
	}

	~FCachedOSVeryLargePageAllocator() = default;

	void* Allocate(SIZE_T Size, uint32 AllocationHint = 0, UE::FPlatformRecursiveMutex* Mutex = nullptr);

	void Free(void* Ptr, SIZE_T Size, UE::FPlatformRecursiveMutex* Mutex = nullptr, bool ThreadIsTimeCritical = false);

	void FreeAll(UE::FPlatformRecursiveMutex* Mutex = nullptr);

	// Refresh cached os allocator if needed. Will preallocate / reduce backstore if preallocation is enabled
	void Refresh();

	void UpdateStats();

	uint64 GetCachedFreeTotal() const
	{
		return CachedFree + CachedOSPageAllocator.GetCachedFreeTotal();
	}

	uint64 GetCachedImmediatelyFreeable() const
	{
		return ImmediatelyFreeable + CachedOSPageAllocator.GetCachedImmediatelyFreeable();
	}

	inline bool IsSmallBlockAllocation(const void* Ptr) const
	{
		if (((uintptr_t)Ptr - AddressSpaceReserved) < AddressSpaceToReserveForSmallPool)
		{
			return true;
		}
		return false;
	}

private:
	struct FLargePage : public TIntrusiveLinkedList<FLargePage>
	{
		uintptr_t	FreeSubPages[NumberOfSubPagesPerLargePage];
		int32		NumberOfFreeSubPages;
		uint32		AllocationHint;

		uintptr_t	BaseAddress;

		void Init(void* InBaseAddress)
		{
			BaseAddress = (uintptr_t)InBaseAddress;
			NumberOfFreeSubPages = NumberOfSubPagesPerLargePage;
			uintptr_t Ptr = BaseAddress;
			for (int i = 0; i < NumberOfFreeSubPages; i++)
			{
				FreeSubPages[i] = Ptr;
				Ptr += SizeOfSubPage;
			}
		}

		void Free(void* Ptr)
		{
			FreeSubPages[NumberOfFreeSubPages++] = (uintptr_t)Ptr;
		}

		void* Allocate()
		{
			void* ret = nullptr;
			if (NumberOfFreeSubPages)
			{
				ret = (void*)FreeSubPages[--NumberOfFreeSubPages];
			}
			return ret;
		}
	};

	void Init();
	void ShrinkEmptyBackStore(int32 NewEmptyBackStoreSize, FMemory::AllocationHints AllocationHint);
	FLargePage* GetOrAllocLargePage(uint32 AllocationHint, UE::FPlatformRecursiveMutex* Mutex);
	FLargePage* AllocNewLargePage(uint32 AllocationHint, UE::FPlatformRecursiveMutex* Mutex, bool& bOutCommitFailure);

	bool bEnabled;
	uintptr_t	AddressSpaceReserved;
	uintptr_t	AddressSpaceReservedEndSmallPool;
	uintptr_t	AddressSpaceReservedEnd;
	uint64		CachedFree;
	uint64		ImmediatelyFreeable;		// the amount of memory that can be immediately returned to the OS
	int32		EmptyBackStoreCount[FMemory::AllocationHints::Max];
	int32		CommittedLargePagesCount[FMemory::AllocationHints::Max];

	FPlatformMemory::FPlatformVirtualMemoryBlock Block;

	FLargePage* FreeLargePagesHead[FMemory::AllocationHints::Max];				// no backing store

	FLargePage* UsedLargePagesHead[FMemory::AllocationHints::Max];				// has backing store and is full

	FLargePage* UsedLargePagesWithSpaceHead[FMemory::AllocationHints::Max];	// has backing store and still has room

	FLargePage*	EmptyButAvailableLargePagesHead[FMemory::AllocationHints::Max];	// has backing store and is empty

	FLargePage	LargePagesArray[NumberOfLargePages];

	TCachedOSPageAllocator<CACHEDOSVERYLARGEPAGEALLOCATOR_MAX_CACHED_OS_FREES, CACHEDOSVERYLARGEPAGEALLOCATOR_BYTE_LIMIT> CachedOSPageAllocator;

};
CORE_API extern bool GEnableVeryLargePageAllocator;

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "HAL/CriticalSection.h"
#endif

#endif // UE_USE_VERYLARGEPAGEALLOCATOR
