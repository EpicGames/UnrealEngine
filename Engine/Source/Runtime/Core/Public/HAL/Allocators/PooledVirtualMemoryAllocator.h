// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/Allocators/CachedOSPageAllocator.h"
#include "HAL/PlatformMutex.h"

/**
 * This class pools OS allocations made from FMallocBinned2.
 * 
 * The class fulfills FMallocBinned2 requirements of returning a 64KB-aligned address
 * and it also avoids fragmenting the memory into too many small VMAs (virtual memory areas).
 * 
 * The logic is like follows:
 * 
 * There are N buckets that represent allocation sizes from 64KB to N*64 KB. Each bucket is a linked list 
 * of pools that hold varied number of same-sized allocations (called blocks). Each bucket starts empty (linked list is empty).
 * 
 * Whenever an allocation request arrives, it is first bucketed based on its size (if it is larger than the largest bucket,
 * it is passed through to a caching OS allocator). Then, the linked list of that bucket is walked, and 
 * the allocation is fulfilled by the first pool that has empty "blocks". If there are no such pool, a new pool is created 
 * (with number of blocks being possibly larger than in the existing list, if any), this pool becomes the new head of the 
 * list and the allocation happens there.
 * 
 * Whenever a free request arrives, it is again first bucketed based on the size (which must be passed in and *must* match 
 * allocation size). If it is larger than the largest bucket, it is passed through to a platform function BinnedFreeToOS().
 * Otherwise, the appropriate bucket's list of pools is walked to find the pool where the allocation belongs, and the 
 * appropriate block becomes free (a platform-specific function is called to evict the memory range from the RAM).
 * If this was the last used block in the pool, the whole pool is destroyed and the list shrinks by one.
 * 
 * So, to visualize an example state:
 * 
 *  64KB bucket:  Head -> [ A pool of 200 64KB "blocks", of which 50 are empty ] -> [ A pool of 100 64KB blocks, of which 30 are empty ] -> null
 * 128KB bucket:  Head -> [ A pool of 60 128KB "blocks", of which 25 are empty ] -> null
 * 192KB bucket:  Head -> null
 * 256KB bucket:  Head -> [ A pool of 40 256KB "blocks", of which 10 are empty ] -> [ A pool of 20 256KB blocks, of which 10 are emtpy ] -> [ A pool of 4 256 KB blocks of which 0 are empty ] -> null
 * ...
 *   4MB bucket:  Head -> [ A pool of 2 4MB "blocks", of which 1 is empty ] -> null
 * 
 * Each pool uses one distinct VMA on Linux (or one distinct VirtualAlloc on Windows).
 * 
 * The class also maintains an idea of what current size of each pool (per bucket) should be.
 * Each time we add a new pool to a particular bucket, this size can grow (exponentially or otherwise).
 * Each time we delete a pool from a particular bucket, this size can shrink.
 * The logic that handles that is in DecideOnTheNextPoolSize().
 * 
 * Right now the class is less multithread friendly than it could be. There are per-bucket locks
 * so allocations of different sizes would not block each other if happening on different threads.
 * It is possible to make allocations within one bucket lock-free as well (e.g. with hazard pointers), but 
 * since FMallocBinned2 itself needs a lock to maintain its own structures when making a call here,
 * the point is moot.
 * 
 * This class is somewhat similar to FCachedOSPageAllocator. However, unlike FCOSPA, this is not a cache
 * and we cannot "trim" anything here. Also, it does not make sense to put the global cap on the pooled memory
 * since BinnedAllocFromOS() can support only a limited number of allocations on some platforms.
 * 
 * CachedOSPageAllocator sits "below" this and is used for allocs larger than the largest bucketed.
 */
struct FPooledVirtualMemoryAllocator
{
	FPooledVirtualMemoryAllocator();

	void* Allocate(SIZE_T Size, uint32 AllocationHint = 0, UE::FPlatformRecursiveMutex* Mutex = nullptr);
	void Free(void* Ptr, SIZE_T Size, UE::FPlatformRecursiveMutex* Mutex = nullptr, bool ThreadIsTimeCritical = false);
	void FreeAll(UE::FPlatformRecursiveMutex* Mutex = nullptr);

	/** A structure that describes a pool of a particular size */
	struct FPoolDescriptorBase
	{
		/** Next in the list */
		FPoolDescriptorBase* Next;

		/** Total size to be deallocated, which includes both pool memory and all the descriptor/bookkeeping memory */
		SIZE_T VMSizeDivVirtualSizeAlignment;
	};

	/** Returns free memory in the pools */
	uint64 GetCachedFreeTotal() const;
	uint64 GetCachedImmediatelyFreeable() const
	{
		return GetCachedFreeTotal();
	}

	/** Refresh allocator if needed. (does nothing in that implementation) */
	void Refresh() {}

	/** Update memory stats (does nothing in that implementation) */
	void UpdateStats() {}

	void DumpAllocatorStats(class FOutputDevice& Ar);

private:

	enum Limits
	{
		NumAllocationSizeClasses	= 64,
		MaxAllocationSizeToPool		= NumAllocationSizeClasses * 65536,

		MaxOSAllocCacheSize			= 64 * 1024 * 1024,
		MaxOSAllocsCached			= 64
	};

	/**
	 * Buckets allocations by its size.
	 *
	 * Allocation size class is an index into a number of tables containing per-class elements.
	 * Index of 0 (smallest possible) represents allocations of size 65536 or less,
	 * index of NumAllocationSizeClasses - 1 (largest possible) represents allocations of size
	 * larger or equal than (NumAllocationSizeClasses - 1 * 65536) and smaller than (NumAllocationSizeClasses * 65536).
	 *
	 * This function should not be passed 0 as Size!
	 */
	UE_FORCEINLINE_HINT int32 GetAllocationSizeClass(SIZE_T Size)
	{
		return static_cast<int32>(Size >> 16) + ((Size & 0xFFFF) ? 1 : 0) - 1;
	}

	/**
	* Returns allocation size for a class.
	*/
	UE_FORCEINLINE_HINT SIZE_T CalculateAllocationSizeFromClass(int32 Class)
	{
		return (static_cast<SIZE_T>(Class) + 1) * 65536;
	}

	/** 
	 * We only pool allocations that are divisible by 64KB.
	 * Max allocation size is pooled is NumAllocationSizeClasses * 64KB, after which
	 * we just allocate directly from the OS.
	 *
	 * This array keeps the number of pooled allocations for each
	 * allocation size class that we keep bumping up whenever the pool is exhausted.
	 */
	int32 NextPoolSize[Limits::NumAllocationSizeClasses];

	/** Head of the pool descriptor lists */
	FPoolDescriptorBase* ClassesListHeads[Limits::NumAllocationSizeClasses];

	/** Per-class locks */
	mutable UE::FPlatformRecursiveMutex ClassesLocks[Limits::NumAllocationSizeClasses];

	/** Increases the number of pooled allocations next time we need a pool
	 *
	 * @param bGrowing - if true, we are allocating it, if false, we have just deleted a pool of this size
	 */
	void DecideOnTheNextPoolSize(int32 SizeClass, bool bGrowing);

	/** Allocates a new pool */
	FPoolDescriptorBase* CreatePool(SIZE_T AllocationSize, int32 NumPooledAllocations);

	/** Destroys a pool */
	void DestroyPool(FPoolDescriptorBase* Pool);

	/** Lock for accessing the caching allocator for larger allocs */
	UE::FPlatformRecursiveMutex OsAllocatorCacheLock;

	/** Caching allocator for larger allocs */
	TCachedOSPageAllocator<MaxOSAllocsCached, MaxOSAllocCacheSize> OsAllocatorCache;
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#	include "HAL/CriticalSection.h"
#endif
