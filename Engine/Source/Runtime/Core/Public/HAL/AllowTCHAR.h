// Copyright Epic Games, Inc. All Rights Reserved.

// This header is expected to be included after HAL/HideTCHAR.h - see the comments in that header
// for more details.

// HEADER_UNIT_SKIP - Not included directly

#ifdef UE_HIDETCHAR_INCLUDED
	#undef UE_HIDETCHAR_INCLUDED
#else
	#error "HAL/AllowTCHAR.h included without a corresponding include of HAL/AllowTCHAR.h - check your includes."
#endif

#if !defined(UE_HIDETCHAR_INCLUDED_FIRST)
	#define UE_HIDETCHAR_INCLUDED_FIRST

	// We want UE_SYSTEM_TCHAR to exist after the first definition of TCHAR - even in
	// !PLATFORM_TCHAR_IS_UTF8CHAR mode.
	//
	// TCHAR is expected to be defined inside the first includes we see being wrapped by our
	// HideTCHAR/AllowTCHAR pair, so we will define our type in terms of it, before we undo our hack,
	// if it exists.
	using UE_SYSTEM_TCHAR = TCHAR;
#endif

#if PLATFORM_TCHAR_IS_UTF8CHAR
	#undef TCHAR
#endif
