// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Simply include the interface header - this file could be deleted if all #includes of it were changed to IConsoleManager
// or we could move this into Private, and put the FConsoleManager class back in
#include "HAL/IConsoleManager.h" // IWYU pragma: export
