// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMutex.h"

namespace UE
{

/**
 * Adapter for FPlatformSharedMutex to the API expected for FRWLock.
 */
class FPlatformRWLock final
{
public:
	UE_REWRITE bool TryWriteLock()
	{
		return Mutex.TryLock();
	}

	UE_REWRITE void WriteLock()
	{
		Mutex.Lock();
	}

	UE_REWRITE void WriteUnlock()
	{
		Mutex.Unlock();
	}

	UE_REWRITE bool TryReadLock()
	{
		return Mutex.TryLockShared();
	}

	UE_REWRITE void ReadLock()
	{
		Mutex.LockShared();
	}

	UE_REWRITE void ReadUnlock()
	{
		Mutex.UnlockShared();
	}

private:
	FPlatformSharedMutex Mutex;
};

} // UE

/** Alias for a mutex that supports recursive locking and may not be fair. */
using FCriticalSection = UE::FPlatformRecursiveMutex;

/** Alias for a shared mutex that does not support recursive locking and may not be fair. */
using FRWLock = UE::FPlatformRWLock;

/** Alias for a system-wide (cross-process) mutex that does not support recursive locking and may not be fair. */
using FSystemWideCriticalSection = UE::FPlatformSystemWideMutex;
