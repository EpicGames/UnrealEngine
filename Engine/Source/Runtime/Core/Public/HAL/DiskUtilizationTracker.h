// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef TRACK_DISK_UTILIZATION
#define TRACK_DISK_UTILIZATION 0
#endif

#include "CoreTypes.h"

#if TRACK_DISK_UTILIZATION
#include "HAL/PlatformMisc.h"
#include "HAL/ThreadSafeBool.h"
#include "ProfilingDebugging/CsvProfiler.h"

CSV_DECLARE_CATEGORY_EXTERN(DiskIO);

#ifndef SPEW_DISK_UTILIZATION
#define SPEW_DISK_UTILIZATION 0
#endif // SPEW_DISK_UTILIZATION

struct FDiskUtilizationTracker
{
	struct UtilizationStats
	{
		UtilizationStats() :
			TotalReads(0),
			TotalSeeks(0),
			TotalBytesRead(0),
			TotalSeekDistance(0),
			TotalIOTime(0.0),
			TotalIdleTime(0.0)
		{}

		double GetOverallThroughputBS() const
		{
			return (TotalIOTime + TotalIdleTime) > 0.0 ? double(TotalBytesRead) / (TotalIOTime + TotalIdleTime) : 0.0;
		}

		double GetOverallThroughputMBS() const
		{
			return GetOverallThroughputBS() / (1024.0 * 1024.0);
		}

		double GetReadThrougputBS() const
		{
			return TotalIOTime > 0.0 ? double(TotalBytesRead) / TotalIOTime : 0.0;
		}

		double GetReadThrougputMBS() const
		{
			return GetReadThrougputBS() / (1024.0 * 1024.0);
		}

		double GetTotalIdleTimeInSeconds() const
		{
			return TotalIdleTime;
		}

		double GetTotalIOTimeInSeconds() const
		{
			return TotalIOTime;
		}

		double GetPercentTimeIdle() const
		{
			double TotalTime = TotalIOTime + TotalIdleTime;

			return TotalTime > 0.0 ? (100.0f * TotalIdleTime) / TotalTime : 0.0;
		}

		void Reset()
		{
			TotalReads = 0;
			TotalSeeks = 0;
			TotalBytesRead = 0;
			TotalSeekDistance = 0;
			TotalIOTime = 0.0;
			TotalIdleTime = 0.0;
		}

		void Dump() const;

		uint64 TotalReads;
		uint64 TotalSeeks;

		uint64 TotalBytesRead;
		uint64 TotalSeekDistance;

		double TotalIOTime;
		double TotalIdleTime;
	};

	UtilizationStats LongTermStats;
	UtilizationStats ShortTermStats;

	FCriticalSection CriticalSection;

	uint64 IdleStartCycle;
	uint64 ReadStartCycle;

	uint64 InFlightBytes;
	int32  InFlightReads;

	FThreadSafeBool bResetShortTermStats;

	FDiskUtilizationTracker() :
		IdleStartCycle(0),
		ReadStartCycle(0),
		InFlightBytes(0),
		InFlightReads(0)
	{
	}

	void StartRead(uint64 InReadBytes, uint64 InSeekDistance = 0);
	void FinishRead();

	uint32 GetOutstandingRequests() const
	{
		return InFlightReads;
	}

	const struct UtilizationStats& GetLongTermStats() const
	{
		return LongTermStats;
	}

	const struct UtilizationStats& GetShortTermStats() const
	{
		return ShortTermStats;
	}

	void ResetShortTermStats()
	{
		bResetShortTermStats = true;
	}

private:
	static float GetThrottleRateMBS();
	static constexpr float PrintFrequencySeconds = 0.5f;

	void MaybePrint();
};

extern FDiskUtilizationTracker GDiskUtilizationTracker;

struct FScopedDiskUtilizationTracker
{
	FScopedDiskUtilizationTracker(uint64 InReadBytes, uint64 InSeekDistance) 
	{
		GDiskUtilizationTracker.StartRead(InReadBytes, InSeekDistance);
	}

	~FScopedDiskUtilizationTracker()
	{
		GDiskUtilizationTracker.FinishRead();
	}
};

#else

struct FScopedDiskUtilizationTracker
{
	FScopedDiskUtilizationTracker(uint64 Size, uint64 SeekDistance)
	{
	}
};

#endif // TRACK_DISK_UTILIZATION