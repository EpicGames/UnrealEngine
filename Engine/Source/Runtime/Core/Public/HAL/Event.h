// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/NumericLimits.h"
#include "Misc/IntrusiveUnsetOptionalState.h"
#include "Misc/Timespan.h"
#include "Templates/Atomic.h"
#include "Templates/SharedPointer.h"

/**
 * Interface for waitable events.
 *
 * This interface has platform-specific implementations that are used to wait for another
 * thread to signal that it is ready for the waiting thread to do some work. It can also
 * be used for telling groups of threads to exit.
 * 
 * Consider using FEventRef as a safer and more convenient alternative.
 */
class FEvent
{
public:

	/**
	 * Creates the event.
	 *
	 * Manually reset events stay triggered until reset.
	 * Named events share the same underlying event.
	 *
	 * @param bIsManualReset Whether the event requires manual reseting or not.
	 * @return true if the event was created, false otherwise.
	 */
	// Message to others in the future, don't try to delete this function as it isn't exactly deprecated,
	// but it should only ever be called from TEventPool::GetEventFromPool()/FSafeRecyclableEvent::GetRawEvent()
	UE_DEPRECATED(5.0, "Direct creation of FEvent is discouraged for performance reasons. Please use FPlatformProcess::GetSynchEventFromPool/ReturnSynchEventToPool.")
	virtual bool Create( bool bIsManualReset = false ) = 0;

	/**
	 * Whether the signaled state of this event needs to be reset manually.
	 *
	 * @return true if the state requires manual resetting, false otherwise.
	 * @see Reset
	 */
	virtual bool IsManualReset() = 0;

	/**
	 * Triggers the event so any waiting threads are activated.
	 *
	 * @see IsManualReset, Reset
	 */
	virtual void Trigger() = 0;

	/**
	 * Resets the event to an untriggered (waitable) state.
	 *
	 * @see IsManualReset, Trigger
	 */
	virtual void Reset() = 0;

	/**
	 * Waits the specified amount of time for the event to be triggered.
	 *
	 * A wait time of MAX_uint32 is treated as infinite wait.
	 *
	 * @param WaitTime The time to wait (in milliseconds).
	 * @param bIgnoreThreadIdleStats If true, ignores ThreadIdleStats
	 * @return true if the event was triggered, false if the wait timed out.
	 */
	virtual bool Wait( uint32 WaitTime, const bool bIgnoreThreadIdleStats = false ) = 0;

	/**
	 * Waits an infinite amount of time for the event to be triggered.
	 *
	 * @return true if the event was triggered.
	 */
	bool Wait()
	{
		return Wait(MAX_uint32);
	}

	/**
	 * Waits the specified amount of time for the event to be triggered.
	 *
	 * @param WaitTime The time to wait.
	 * @param bIgnoreThreadIdleStats If true, ignores ThreadIdleStats
	 * @return true if the event was triggered, false if the wait timed out.
	 */
	bool Wait( const FTimespan& WaitTime, const bool bIgnoreThreadIdleStats = false )
	{
		check(WaitTime.GetTicks() >= 0);
		return Wait((uint32)FMath::Clamp<int64>(WaitTime.GetTicks() / ETimespan::TicksPerMillisecond, 0, MAX_uint32), bIgnoreThreadIdleStats);
	}

	/** Default constructor. */
	FEvent()
		: EventId( 0 )
		, EventStartCycles( 0 )
	{}

	/** Virtual destructor. */
	virtual ~FEvent() = default; 

	// DO NOT MODIFY THESE

	/** Advances stats associated with this event. Used to monitor wait->trigger history. */
	CORE_API void AdvanceStats();

protected:
	/** Sends to the stats a special messages which encodes a wait for the event. */
	CORE_API void WaitForStats();

	/** Send to the stats a special message which encodes a trigger for the event. */
	CORE_API void TriggerForStats();

	/** Resets start cycles to 0. */
	CORE_API void ResetForStats();

	/** Counter used to generate an unique id for the events. */
	static CORE_API TAtomic<uint32> EventUniqueId;

	/** An unique id of this event. */
	uint32 EventId;

	/** Greater than 0, if the event called wait. */
	TAtomic<uint32> EventStartCycles;
};

enum class EEventMode { AutoReset, ManualReset };

/**
 * RAII-style pooled `FEvent`
 *
 * non-copyable, non-movable
 */
class FEventRef final
{
public:
	CORE_API explicit FEventRef(EEventMode Mode = EEventMode::AutoReset);

	CORE_API ~FEventRef();

	FEventRef(const FEventRef&) = delete;
	FEventRef& operator=(const FEventRef&) = delete;
	FEventRef(FEventRef&& Other) = delete;
	FEventRef& operator=(FEventRef&& Other) = delete;

	FEvent* operator->() const
	{
		return Event;
	}

	FEvent* Get()
	{
		return Event;
	}

private:
	FEvent* Event;
};

/**
 * RAII-style shared and pooled `FEvent`
 */
class FSharedEventRef final
{
public:
	CORE_API explicit FSharedEventRef(EEventMode Mode = EEventMode::AutoReset);

	FEvent* operator->() const
	{
		return Ptr.Get();
	}

	// Support intrusive TOptional state
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = FSharedEventRef;

	explicit FSharedEventRef(FIntrusiveUnsetOptionalState Tag) 
		: Ptr(Tag)
	{}
	bool operator==(FIntrusiveUnsetOptionalState Tag) const
	{
		return Ptr == Tag;
	}

private:
	TSharedPtr<FEvent> Ptr;
};
