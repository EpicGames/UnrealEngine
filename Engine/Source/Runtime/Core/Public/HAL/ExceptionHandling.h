// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "CoreTypes.h"

struct FProgramCounterSymbolInfo;

/** Whether we should generate crash reports even if the debugger is attached. */
extern CORE_API bool GAlwaysReportCrash;

/** Whether to use ClientReportClient rather than AutoReporter. */
extern CORE_API bool GUseCrashReportClient;

extern CORE_API TCHAR MiniDumpFilenameW[1024];

// #CrashReport: 2014-09-11 Move to PlatformExceptionHandling
#if PLATFORM_WINDOWS
#include "Windows/WindowsSystemIncludes.h"

#include <excpt.h>

// #CrashReport: 2014-10-09 These methods are specific to windows, remove from here.
extern CORE_API int32 ReportCrash( Windows::LPEXCEPTION_POINTERS ExceptionInfo );
extern CORE_API void ReportAssert(const TCHAR* ErrorMessage, void* ProgramCounter);
extern CORE_API void ReportGPUCrash(const TCHAR* ErrorMessage, void* ProgramCounter);
extern CORE_API void ReportEnsure(const TCHAR* ErrorMessage, void* ProgramCounter);
extern CORE_API void ReportStall(const TCHAR* ErrorMessage, uint32 HitchThreadId);
extern CORE_API void ReportHang(const TCHAR*, const uint64* StackFrames, int32 NumStackFrames, uint32 HungThreadId);
#elif PLATFORM_MAC
// #CrashReport: 2014-10-09 Should be move to another file
#include <signal.h>

extern CORE_API int32 ReportCrash(ucontext_t *Context, int32 Signal, struct __siginfo* Info);
extern CORE_API void ReportAssert(const TCHAR* ErrorMessage, void* ProgramCounter);
extern CORE_API void ReportGPUCrash(const TCHAR* ErrorMessage, void* ProgramCounter);
extern CORE_API void ReportEnsure(const TCHAR* ErrorMessage, void* ProgramCounter);
extern CORE_API void ReportHang(const TCHAR*, const uint64* StackFrames, int32 NumStackFrames, uint32 HungThreadId);
#elif PLATFORM_UNIX
extern CORE_API void ReportAssert(const TCHAR* ErrorMessage, void* ProgramCounter);
extern CORE_API void ReportGPUCrash(const TCHAR* ErrorMessage, void* ProgramCounter);
extern CORE_API void ReportEnsure(const TCHAR* ErrorMessage, void* ProgramCounter);
extern CORE_API void ReportStall(const TCHAR* ErrorMessage, uint32 ThreadId);
extern CORE_API void ReportHang(const TCHAR*, const uint64* StackFrames, int32 NumStackFrames, uint32 HungThreadId);
#endif


extern CORE_API void ReportInteractiveEnsure(const TCHAR* InMessage);
extern CORE_API bool IsInteractiveEnsureMode();