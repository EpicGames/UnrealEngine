// Copyright Epic Games, Inc. All Rights Reserved.

#if !defined(FMEMORY_INLINE_GMalloc)
#	error "FMEMORY_INLINE_GMalloc should be defined before including this file. Possibly FMemory.inl is included directly instead of including Memory.h"
#endif

#include "HAL/LowLevelMemTracker.h"
#include "AutoRTFM.h"

struct FMemory;
struct FScopedMallocTimer;

void FMemory_FreeInline(void* Original);
SIZE_T FMemory_GetAllocSizeInline(void* Original);

FORCEINLINE void* FMemory_MallocInline(SIZE_T Count, uint32 Alignment)
{
	// AutoRTFM: For non-transactional code, all of these calls optimize away and the
	// behavior is the same as it always has been.
	// For transactional code, we call the allocator in the 'open' as an optimization, so that
	// we don't end up keeping track of the writes to the allocator's internal data structures.
	// This is because allocators are already transactional - malloc can be rolled back by
	// calling free.
	void* Ptr = nullptr;

	UE_AUTORTFM_OPEN_NO_VALIDATION
	{
		if (UNLIKELY(!FMEMORY_INLINE_GMalloc))
		{
			Ptr = FMemory::MallocExternal(Count, Alignment);
		}
		else
		{
			DoGamethreadHook(0);
			FScopedMallocTimer Timer(0);
			Ptr = FMEMORY_INLINE_GMalloc->Malloc(Count, Alignment);
		}
		// optional tracking of every allocation
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, Ptr, Count, ELLMTag::Untagged, ELLMAllocType::FMalloc));
	};

	// AutoRTFM: This is a no-op for non-transactional code.
	// For transactional code, this defers a call to Free if the transaction aborts,
	// so that rolling back this allocation will end up freeing the memory.
	UE_AUTORTFM_ONABORT(Ptr)
	{
		// Disable the code analysis warning that complains that Free is being passed
		// a pointer that may be null. Free explicitly handles this case already.
		FMemory::Free(Ptr); //-V575
	};

	return AutoRTFM::DidAllocate(Ptr, Count);
}

FORCEINLINE void* FMemory_ReallocInline(void* Original, SIZE_T Count, uint32 Alignment)
{
	if (AutoRTFM::IsClosed())
	{
		// AutoRTFM: For transactional code, we have to do a little dance to handle Realloc
		// properly. We turn realloc into Malloc + Memcpy + Free and never call into the
		// underlying allocator's realloc implementation. That's required, because if we
		// were to call into actual realloc, it could end up freeing the old memory. And
		// if we then aborted our transaction, we would end up rolling back pointers to
		// point to that old allocation, and it wouldn't be possible to get back that original
		// allocation at the same address.
		// 
		// There is an opportunity here, in that if the original pointer was allocated within
		// this transaction, then the above doesn't apply since rolling back the transaction
		// would free that memory and the resulting heap wouldn't point at that memory. So
		// if we new that the Original pointer was allocated in this transaction, we could
		// call into the underlying realloc - however we would also have to account for the
		// malloc deferring a call to free, so we would also have to erase that call to free.

		void* Ptr = nullptr;

		// Depending on the underlying implementation `Malloc` here, even if `Count` is zero,
		// could do an actual allocation (it is implementation-defined what occurs). So
		// instead, since we are fine to return null with a `Count` of zero, we check for
		// that case and skip the `Malloc` call entirely.
		if (Count > 0)
		{
			Ptr = FMemory::Malloc(Count, Alignment);

			if (!Ptr)
			{
				return nullptr;
			}
		}

		if (Original)
		{
			if (Ptr)
			{
				SIZE_T OriginalCount = FMemory::GetAllocSize(Original);
				SIZE_T CopyCount = FGenericPlatformMath::Min(Count, OriginalCount); // handle the case where the new size is smaller

				FMemory::Memcpy(Ptr, Original, CopyCount);
			}
			
			FMemory::Free(Original);
		}

		return Ptr;
	}

	// optional tracking -- a realloc with an Original pointer of null is equivalent
	// to malloc() so there's nothing to free
	LLM_REALLOC_SCOPE(Original);
	LLM_IF_ENABLED(if (Original != nullptr) FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, Original, ELLMAllocType::FMalloc));

	void* Ptr;
	if (UNLIKELY(!FMEMORY_INLINE_GMalloc))
	{
		Ptr = FMemory::ReallocExternal(Original, Count, Alignment);
	}
	else
	{
		DoGamethreadHook(1);
		FScopedMallocTimer Timer(1);
		Ptr = FMEMORY_INLINE_GMalloc->Realloc(Original, Count, Alignment);
	}

	if (Ptr != Original)
	{
		// If the pointer we've got back from realloc is new memory, we are
		// assuming the old memory was free'd.
		AutoRTFM::DidFree(Original);
	}

	// optional tracking of every allocation - a realloc with a Count of zero is equivalent to a call 
	// to free() and will return a null pointer which does not require tracking. If realloc returns null
	// for some other reason (like failure to allocate) there's also no reason to track it
	LLM_IF_ENABLED(if (Ptr != nullptr) FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, Ptr, Count, ELLMTag::Untagged, ELLMAllocType::FMalloc));

	return Ptr;
}

FORCEINLINE void FMemory_FreeInline(void* Original)
{
	if (!Original)
	{
		FScopedMallocTimer Timer(3);
		return;
	}

	// AutoRTFM: For transactional code, in order to support the transaction 
	// aborting and needing to 'roll back' the Free, we defer the actual
	// free until commit time.
	UE_AUTORTFM_ONCOMMIT(=)
	{
		// optional tracking of every allocation
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, Original, ELLMAllocType::FMalloc));

		if (UNLIKELY(!FMEMORY_INLINE_GMalloc))
		{
			FMemory::FreeExternal(Original);
			return;
		}
		DoGamethreadHook(2);
		FScopedMallocTimer Timer(2);
		FMEMORY_INLINE_GMalloc->Free(Original);

		AutoRTFM::DidFree(Original);
	};
}

FORCEINLINE SIZE_T FMemory_GetAllocSizeInline(void* Original)
{
	const bool bWasClosed = AutoRTFM::IsClosed();

	SIZE_T Result;

	UE_AUTORTFM_OPEN_NO_VALIDATION
	{
		if (UNLIKELY(!FMEMORY_INLINE_GMalloc))
		{
			Result = FMemory::GetAllocSizeExternal(Original);
		}
		else
		{
			SIZE_T Size = 0;
			const bool bGotSize = FMEMORY_INLINE_GMalloc->GetAllocationSize(Original, Size);
			Result = bGotSize ? Size : 0;

			// This folds away at compile time so that the check is only ever performed inside transactional
			// code paths. The check is to ensure that the allocator used will return the correct allocation
			// size, which is a cornerstone requirement for AutoRTFM to function.
			if (bWasClosed)
			{
				checkf(bGotSize, TEXT("For AutoRTFM to function it must be able to get the size of an allocation"));
			}
		}
	};

	return Result;
}

FORCEINLINE void* FMemory_MallocZeroedInline(SIZE_T Count, uint32 Alignment)
{
	void* Ptr = nullptr; // Silence bogus static analysis warnings.

	// AutoRTFM: For non-transactional code, all of these calls optimize away and the
	// behavior is the same as it always has been.
	// For transactional code, we call the allocator in the 'open' as an optimization, so that
	// we don't end up keeping track of the writes to the allocator's internal data structures.
	// This is because allocators are already transactional - malloc can be rolled back by
	// calling free.
	UE_AUTORTFM_OPEN_NO_VALIDATION
	{
		if (UNLIKELY(!FMEMORY_INLINE_GMalloc))
		{
			Ptr = FMemory::MallocZeroedExternal(Count, Alignment);
		}
		else
		{
			DoGamethreadHook(0);
			FScopedMallocTimer Timer(0);
			Ptr = FMEMORY_INLINE_GMalloc->MallocZeroed(Count, Alignment);
		}
		// optional tracking of every allocation
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, Ptr, Count, ELLMTag::Untagged, ELLMAllocType::FMalloc));
	};

	// AutoRTFM: This is a no-op for non-transactional code.
	// For transactional code, this defers a call to Free if the transaction aborts,
	// so that rolling back this allocation will end up freeing the memory.
	UE_AUTORTFM_ONABORT(Ptr)
	{
		// Disable the code analysis warning that complains that Free is being passed
		// a pointer that may be null. Free explicitly handles this case already.
		FMemory::Free(Ptr); //-V575
	};

	return AutoRTFM::DidAllocate(Ptr, Count);
}

FORCEINLINE SIZE_T FMemory_QuantizeSizeInline(SIZE_T Count, uint32 Alignment)
{
	SIZE_T Result;

	UE_AUTORTFM_OPEN_NO_VALIDATION
	{
		if (UNLIKELY(!FMEMORY_INLINE_GMalloc))
		{
			Result = Count;
		}
		else
		{
			Result = FMEMORY_INLINE_GMalloc->QuantizeSize(Count, Alignment);
		}
	};

	return Result;
}