// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Misc/App.h"
#include "Misc/OutputDeviceConsole.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/FeedbackContext.h"

class FFeedbackContextAnsi : public FFeedbackContext
{
public:
	FContextSupplier*	Context = nullptr;
	FOutputDevice*		AuxOut = nullptr;

	CORE_API static void LocalPrint(const TCHAR* Str);

	CORE_API void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override;
	CORE_API void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time) override;
	CORE_API void SerializeRecord(const UE::FLogRecord& Record) override;

	CORE_API bool YesNof(const FText& Question) override;

	FContextSupplier* GetContext() const override { return Context; }
	void SetContext(FContextSupplier* InContext) override { Context = InContext; }

private:
	bool IsUsingLocalPrint() const;
};
