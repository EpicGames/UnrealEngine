// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Misc/DateTime.h"
#include "GenericPlatform/GenericPlatformFile.h"

// Maximum length of any filename.  For now, we have no restriction. We would probably use shortening rules if we have to.
#define MAX_UNREAL_FILENAME_LENGTH_DEPRECATED (PLATFORM_MAX_FILEPATH_LENGTH_DEPRECATED)


enum EFileWrite
{
	FILEWRITE_None				= 0x00,
	FILEWRITE_NoFail            = 0x01,
	FILEWRITE_NoReplaceExisting = 0x02,
	FILEWRITE_EvenIfReadOnly    = 0x04,
	FILEWRITE_Append			= 0x08,
	FILEWRITE_AllowRead			= 0x10,
	FILEWRITE_Silent			= 0x20
};


enum EFileRead
{
	FILEREAD_None				= 0x00,
	FILEREAD_NoFail             = 0x01,
	FILEREAD_Silent				= 0x02,
	FILEREAD_AllowWrite			= 0x04,
};


enum ECopyResult
{
	COPY_OK						= 0x00,
	COPY_Fail					= 0x01,
	COPY_Canceled				= 0x02,
};


struct FCopyProgress
{
	virtual bool Poll( float Fraction )=0;
};


enum EFileOpenFlags
{
	IO_READ			= 0x01,					// Open for reading
	IO_WRITE		= 0x02,					// Open for writing
	IO_APPEND		= 0x40,					// When writing, keep the existing data, set the filepointer to the end of the existing data
};

class IFileManager
{
protected:

	/** Construtor. */
	IFileManager() {}

public:

	/** Singleton access, platform specific, also calls PreInit() **/
	static CORE_API IFileManager& Get();

	/** Allow the file manager to handle the commandline */
	virtual void ProcessCommandLineOptions() = 0;

	/** Enables/disables the sandbox, if it is being used */
	virtual void SetSandboxEnabled(bool bInEnabled) = 0;
	/** Returns whether the sandbox is enabled or not */
	virtual bool IsSandboxEnabled() const = 0;

	/**
	 * Opens a file for reading and create an FArchive which can be used to read from it.
	 *
	 * @param Filename	Path to the file to open
	 * @param ReadFlags	An optional bitfield of optional flags. For flag values see @See EFileRead
	 *
	 * @return Returns a pointer to an FArchive when successful and a nullptr if the operation failed.
	 * Note that it is up to the caller to delete the FArchive when done.
	 */
	virtual FArchive* CreateFileReader( const TCHAR* Filename, uint32 ReadFlags=0 )=0;

	/**
	 * Opens a file for writing and create an FArchive which can be used to write to it.
	 *
	 * @param Filename		Path to the desired location of the file
	 * @param WriteFlags	An optional bitfield of optional flags. For flag values see @See EFileWrite
	 *
	 * @return Returns a pointer to an FArchive when successful and a nullptr if the operation failed.
	 * Note that it is up to the caller to delete the FArchive when done.
	 */
	virtual FArchive* CreateFileWriter( const TCHAR* Filename, uint32 WriteFlags=0 )=0;

	// If you're writing to a debug file, you should use CreateDebugFileWriter, and wrap the calling code in #if ALLOW_DEBUG_FILES.
#if ALLOW_DEBUG_FILES
	virtual FArchive* CreateDebugFileWriter(const TCHAR* Filename, uint32 WriteFlags=0 ) = 0;
#endif

	/** Checks if a file is read-only. */
	virtual bool IsReadOnly( const TCHAR* Filename )=0;

	/** Deletes a file. */
	virtual bool Delete( const TCHAR* Filename, bool RequireExists=0, bool EvenReadOnly=0, bool Quiet=0 )=0;

	/** Copies a file. */
	virtual uint32 Copy( const TCHAR* Dest, const TCHAR* Src, bool Replace=1, bool EvenIfReadOnly=0, bool Attributes=0, FCopyProgress* Progress = nullptr, EFileRead ReadFlags=FILEREAD_None, EFileWrite WriteFlags=FILEWRITE_None)=0; // utility

	/** Moves/renames a file. */
	virtual bool Move( const TCHAR* Dest, const TCHAR* Src, bool Replace=1, bool EvenIfReadOnly=0, bool Attributes=0, bool bDoNotRetryOrError=0 )=0;

	/** Checks if a file exists */
	virtual bool FileExists( const TCHAR* Filename )=0;

	/** Checks if a directory exists. */
	virtual bool DirectoryExists( const TCHAR* InDirectory )=0;

	/** Creates a directory. */
	virtual bool MakeDirectory( const TCHAR* Path, bool Tree=0 )=0;

	/** Deletes a directory. */
	virtual bool DeleteDirectory( const TCHAR* Path, bool RequireExists=0, bool Tree=0 )=0;

	/** Return the stat data for the given file or directory. Check the FFileStatData::bIsValid member before using the returned data */
	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) = 0;

	/** Finds file or directories. */
	virtual void FindFiles( TArray<FString>& FileNames, const TCHAR* Filename, bool Files, bool Directories)=0;

	/**
	 * Finds all the files within the given directory, with optional file extension filter.
	 *
	 * @param[out] FoundFiles All the files that matched the optional FileExtension filter, or all files if none was specified.
	 *
	 * @param[in] Directory The absolute path to the directory to search. Ex: "C:\UE4\Pictures"
	 *
	 * @param[in] FileExtension If FileExtension is NULL, or an empty string "" then all files are found.
	 *			Otherwise FileExtension can be of the form .EXT or just EXT and only files with that extension will be returned.
	 */
	virtual void FindFiles(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension = nullptr) = 0;

	/** Finds file or directories recursively. */
	virtual void FindFilesRecursive( TArray<FString>& FileNames, const TCHAR* StartDirectory, const TCHAR* Filename, bool Files, bool Directories, bool bClearFileNames=true) = 0; // utility

	/** 
	 * Call the Visit function of the visitor once for each file or directory in a single directory. This function does not explore subdirectories.
	 * @param Directory		The directory to iterate the contents of.
	 * @param Visitor		Visitor to call for each element of the directory
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	virtual bool IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) = 0;

	/**
	 * Call the visitor once for each file or directory in a single directory. This function does not explore subdirectories.
	 * @param Directory		The directory to iterate the contents of.
	 * @param Visitor		Visitor to call for each element of the directory (see FDirectoryVisitor::Visit for the signature)
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	virtual bool IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitorFunc Visitor) = 0;

	/** 
	 * Call the Visit function of the visitor once for each file or directory in a directory tree. This function explores subdirectories.
	 * @param Directory		The directory to iterate the contents of, recursively.
	 * @param Visitor		Visitor to call for each element of the directory and each element of all subdirectories.
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	virtual bool IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) = 0;

	/**
	 * Call the Visit function of the visitor once for each file or directory in a directory tree. This function explores subdirectories.
	 * @param Directory		The directory to iterate the contents of, recursively.
	 * @param Visitor		Visitor to call for each element of the directory and each element of all subdirectories (see FDirectoryVisitor::Visit for the signature).
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	virtual bool IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitorFunc Visitor) = 0;

	/** 
	 * Call the Visit function of the visitor once for each file or directory in a single directory. This function does not explore subdirectories.
	 * @param Directory		The directory to iterate the contents of.
	 * @param Visitor		Visitor to call for each element of the directory
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	virtual bool IterateDirectoryStat(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) = 0;

	/**
	 * Call the visitor once for each file or directory in a single directory. This function does not explore subdirectories.
	 * @param Directory		The directory to iterate the contents of.
	 * @param Visitor		Visitor to call for each element of the directory (see FDirectoryStatVisitor::Visit for the signature)
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	virtual bool IterateDirectoryStat(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitorFunc Visitor) = 0;

	/** 
	 * Call the Visit function of the visitor once for each file or directory in a directory tree. This function explores subdirectories.
	 * @param Directory		The directory to iterate the contents of, recursively.
	 * @param Visitor		Visitor to call for each element of the directory and each element of all subdirectories.
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	virtual bool IterateDirectoryStatRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) = 0;

	/**
	 * Call the Visit function of the visitor once for each file or directory in a directory tree. This function explores subdirectories.
	 * @param Directory		The directory to iterate the contents of, recursively.
	 * @param Visitor		Visitor to call for each element of the directory and each element of all subdirectories (see FDirectoryStatVisitor::Visit for the signature).
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	virtual bool IterateDirectoryStatRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitorFunc Visitor) = 0;

	/** Gets the age of a file measured in seconds. */
	virtual double GetFileAgeSeconds( const TCHAR* Filename )=0;

	/** 
	 * @return the modification time of the given file (or FDateTime::MinValue() on failure)
	 */
	virtual FDateTime GetTimeStamp( const TCHAR* Path ) = 0;

	/**
	 * @param[in] PathA The first given file
	 * @param[in] PathB The second given file
	 * @param[out] OutTimeStampA the modification time of the first given file (or FDateTime::MinValue() on failure)
	 * @param[out] OutTimeStampB the modification time of the second given file (or FDateTime::MinValue() on failure)
	*/
	virtual void GetTimeStampPair(const TCHAR* PathA, const TCHAR* PathB, FDateTime& OutTimeStampA, FDateTime& OutTimeStampB) = 0;

	/** 
	 * Sets the modification time of the given file
	 */
	virtual bool SetTimeStamp( const TCHAR* Path, FDateTime TimeStamp ) = 0;

	/** 
	 * @return the last access time of the given file (or FDateTime::MinValue() on failure)
	 */
	virtual FDateTime GetAccessTimeStamp( const TCHAR* Filename ) = 0;

	/**
	 * Converts passed in filename to use a relative path.
	 *
	 * @param	Filename	filename to convert to use a relative path
	 * 
	 * @return	filename using relative path
	 */
	virtual FString ConvertToRelativePath( const TCHAR* Filename ) = 0;

	/**
	 * Converts passed in filename to use an absolute path (for reading)
	 *
	 * @param	Filename	filename to convert to use an absolute path, safe to pass in already using absolute path
	 * 
	 * @return	filename using absolute path
	 */
	virtual FString ConvertToAbsolutePathForExternalAppForRead( const TCHAR* Filename ) = 0;

	/**
	 * Converts passed in filename to use an absolute path (for writing)
	 *
	 * @param	Filename	filename to convert to use an absolute path, safe to pass in already using absolute path
	 * 
	 * @return	filename using absolute path
	 */
	virtual FString ConvertToAbsolutePathForExternalAppForWrite( const TCHAR* Filename ) = 0;

	/**
	 *	Returns the size of a file. (Thread-safe)
	 *
	 *	@param Filename		Platform-independent Unreal filename.
	 *	@return				File size in bytes or INDEX_NONE if the file didn't exist.
	 **/
	virtual int64 FileSize( const TCHAR* Filename )=0;

	/**
	 * Sends a message to the file server, and will block until it's complete. Will return 
	 * immediately if the file manager doesn't support talking to a server.
	 *
	 * @param Message	The string message to send to the server
	 *
	 * @return			true if the message was sent to server and it returned success, or false if there is no server, or the command failed
	 */
	virtual bool SendMessageToServer(const TCHAR* Message, IPlatformFile::IFileServerMessageHandler* Handler)=0;

	/**
	* For case insensitive filesystems, returns the full path of the file with the same case as in the filesystem.
	*
	* @param Filename	Filename to query
	*
	* @return	Filename with the same case as in the filesystem.
	*/
	virtual FString GetFilenameOnDisk(const TCHAR* Filename) = 0;

	/**
	* Returns if the given path is a symlink. Depending on the OS this can be a soft link, hard link or a junction.
	*
	* @param Path	Path to check if it's a symlink
	*
	* @return	True if symlink, false if not or the symlink check is not implemented. 
	*/
	virtual bool IsSymlink(const TCHAR* Path) = 0;
};
