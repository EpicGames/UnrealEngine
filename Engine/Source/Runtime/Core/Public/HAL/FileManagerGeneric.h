// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/DateTime.h"
#include "Serialization/Archive.h"
#include "Templates/UniquePtr.h"

#ifndef PLATFORM_FILE_READER_BUFFER_SIZE
 #define PLATFORM_FILE_READER_BUFFER_SIZE 1024
#endif

#ifndef PLATFORM_FILE_WRITER_BUFFER_SIZE
 #define PLATFORM_FILE_WRITER_BUFFER_SIZE 4096
#endif 

#ifndef PLATFORM_DEBUG_FILE_WRITER_BUFFER_SIZE
 #define PLATFORM_DEBUG_FILE_WRITER_BUFFER_SIZE 4096
#endif 

/**
 * Base class for file managers.
 *
 * This base class simplifies IFileManager implementations by providing
 * simple, unoptimized implementations of functions whose implementations
 * can be derived from other functions.
 */
class FFileManagerGeneric
	: public IFileManager
{
	// instead of caching the LowLevel, we call the singleton each time to never be incorrect
	UE_FORCEINLINE_HINT IPlatformFile& GetLowLevel() const
	{
		return FPlatformFileManager::Get().GetPlatformFile();
	}

public:

	/**
	 * Default constructor.
	 */
	FFileManagerGeneric( ) { }

	/**
	 * Virtual destructor.
	 */
	virtual ~FFileManagerGeneric( ) = default;

public:

	// IFileManager interface

	CORE_API virtual void ProcessCommandLineOptions() override;

	virtual void SetSandboxEnabled(bool bInEnabled) override
	{
		GetLowLevel().SetSandboxEnabled(bInEnabled);
	}

	virtual bool IsSandboxEnabled() const override
	{
		return GetLowLevel().IsSandboxEnabled();
	}

	FArchive* CreateFileReader( const TCHAR* Filename, uint32 ReadFlags=0 ) override
	{
		return CreateFileReaderInternal( Filename, ReadFlags, PLATFORM_FILE_READER_BUFFER_SIZE );
	}

	FArchive* CreateFileWriter( const TCHAR* Filename, uint32 WriteFlags=0 ) override
	{
		return CreateFileWriterInternal( Filename, WriteFlags, PLATFORM_FILE_WRITER_BUFFER_SIZE );
	}

#if ALLOW_DEBUG_FILES
	FArchive* CreateDebugFileWriter( const TCHAR* Filename, uint32 WriteFlags=0 ) override
	{
		return CreateFileWriterInternal( Filename, WriteFlags, PLATFORM_DEBUG_FILE_WRITER_BUFFER_SIZE );
	}
#endif

	CORE_API bool Delete( const TCHAR* Filename, bool RequireExists=0, bool EvenReadOnly=0, bool Quiet=0 ) override;
	CORE_API bool IsReadOnly( const TCHAR* Filename ) override;
	CORE_API bool Move( const TCHAR* Dest, const TCHAR* Src, bool Replace=1, bool EvenIfReadOnly=0, bool Attributes=0, bool bDoNotRetryOrError=0 ) override;
	CORE_API bool FileExists( const TCHAR* Filename ) override;
	CORE_API bool DirectoryExists(const TCHAR* InDirectory) override;
	CORE_API void FindFiles( TArray<FString>& Result, const TCHAR* Filename, bool Files, bool Directories ) override;
	CORE_API void FindFilesRecursive( TArray<FString>& FileNames, const TCHAR* StartDirectory, const TCHAR* Filename, bool Files, bool Directories, bool bClearFileNames=true) override;
	CORE_API double GetFileAgeSeconds( const TCHAR* Filename ) override;
	CORE_API FDateTime GetTimeStamp( const TCHAR* Filename ) override;
	CORE_API FDateTime GetAccessTimeStamp( const TCHAR* Filename ) override;
	CORE_API void GetTimeStampPair(const TCHAR* PathA, const TCHAR* PathB, FDateTime& OutTimeStampA, FDateTime& OutTimeStampB);
	CORE_API bool SetTimeStamp( const TCHAR* Filename, FDateTime Timestamp ) override;
	CORE_API virtual FString GetFilenameOnDisk(const TCHAR* Filename) override;
	CORE_API virtual bool IsSymlink(const TCHAR* Filename) override;

	CORE_API virtual uint32	Copy( const TCHAR* Dest, const TCHAR* Src, bool Replace = 1, bool EvenIfReadOnly = 0, bool Attributes = 0, FCopyProgress* Progress = nullptr, EFileRead ReadFlags = FILEREAD_None, EFileWrite WriteFlags = FILEWRITE_None ) override;
	CORE_API virtual bool	MakeDirectory( const TCHAR* Path, bool Tree=0 ) override;
	CORE_API virtual bool	DeleteDirectory( const TCHAR* Path, bool RequireExists=0, bool Tree=0 ) override;

	CORE_API virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override;

	/**
	 * Finds all the files within the given directory, with optional file extension filter.
	 *
	 * @param Directory, the absolute path to the directory to search. Ex: "C:\UE4\Pictures"
	 *
	 * @param FileExtension, If FileExtension is NULL, or an empty string "" then all files are found.
	 * 			Otherwise FileExtension can be of the form .EXT or just EXT and only files with that extension will be returned.
	 *
	 * @return FoundFiles, All the files that matched the optional FileExtension filter, or all files if none was specified.
	 */
	CORE_API virtual void FindFiles(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension = nullptr) override;

	/** 
	 * Call the Visit function of the visitor once for each file or directory in a single directory. This function does not explore subdirectories.
	 * @param Directory		The directory to iterate the contents of.
	 * @param Visitor		Visitor to call for each element of the directory
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	CORE_API bool IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override;
	CORE_API bool IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitorFunc Visitor) override;

	/** 
	 * Call the Visit function of the visitor once for each file or directory in a directory tree. This function explores subdirectories.
	 * @param Directory		The directory to iterate the contents of, recursively.
	 * @param Visitor		Visitor to call for each element of the directory and each element of all subdirectories.
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	CORE_API bool IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override;
	CORE_API bool IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitorFunc Visitor) override;

	/** 
	 * Call the Visit function of the visitor once for each file or directory in a single directory. This function does not explore subdirectories.
	 * @param Directory		The directory to iterate the contents of.
	 * @param Visitor		Visitor to call for each element of the directory
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	CORE_API bool IterateDirectoryStat(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) override;
	CORE_API bool IterateDirectoryStat(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitorFunc Visitor) override;

	/** 
	 * Call the Visit function of the visitor once for each file or directory in a directory tree. This function explores subdirectories.
	 * @param Directory		The directory to iterate the contents of, recursively.
	 * @param Visitor		Visitor to call for each element of the directory and each element of all subdirectories.
	 * @return				false if the directory did not exist or if the visitor returned false.
	**/
	CORE_API bool IterateDirectoryStatRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) override;
	CORE_API bool IterateDirectoryStatRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitorFunc Visitor) override;

	/**
	 * Converts passed in filename to use a relative path.
	 *
	 * @param	Filename	filename to convert to use a relative path
	 * @return	filename using relative path
	 */
	static CORE_API FString DefaultConvertToRelativePath( const TCHAR* Filename );

	/**
	 * Converts passed in filename to use a relative path.
	 *
	 * @param	Filename	filename to convert to use a relative path
	 * @return	filename using relative path
	 */
	CORE_API FString ConvertToRelativePath( const TCHAR* Filename ) override;

	/**
	 * Converts passed in filename to use an absolute path (for reading)
	 *
	 * @param	Filename	filename to convert to use an absolute path, safe to pass in already using absolute path
	 * @return	filename using absolute path
	 */
	CORE_API FString ConvertToAbsolutePathForExternalAppForRead( const TCHAR* Filename ) override;

	/**
	 * Converts passed in filename to use an absolute path (for writing)
	 *
	 * @param	Filename	filename to convert to use an absolute path, safe to pass in already using absolute path
	 * @return	filename using absolute path
	 */
	CORE_API FString ConvertToAbsolutePathForExternalAppForWrite( const TCHAR* Filename ) override;

	/**
	 *	Returns the size of a file. (Thread-safe)
	 *
	 *	@param Filename		Platform-independent Unreal filename.
	 *	@return				File size in bytes or INDEX_NONE if the file didn't exist.
	 **/
	CORE_API int64 FileSize( const TCHAR* Filename ) override;

	/**
	 * Sends a message to the file server, and will block until it's complete. Will return 
	 * immediately if the file manager doesn't support talking to a server.
	 *
	 * @param Message	The string message to send to the server
	 * @return			true if the message was sent to server and it returned success, or false if there is no server, or the command failed
	 */
	virtual bool SendMessageToServer(const TCHAR* Message, IPlatformFile::IFileServerMessageHandler* Handler) override
	{
		return GetLowLevel().SendMessageToServer(Message, Handler);
	}

private:
	CORE_API FArchive * CreateFileReaderInternal(const TCHAR* Filename, uint32 ReadFlags, uint32 BufferSize);
	CORE_API FArchive* CreateFileWriterInternal(const TCHAR* Filename, uint32 WriteFlags, uint32 BufferSize);

	/**
	 * Helper called from Copy if Progress is available
	 */
	CORE_API uint32	CopyWithProgress(const TCHAR* InDestFile, const TCHAR* InSrcFile, bool ReplaceExisting, bool EvenIfReadOnly, bool Attributes, FCopyProgress* Progress, EFileRead ReadFlags, EFileWrite WriteFlags);

	CORE_API void FindFilesRecursiveInternal( TArray<FString>& FileNames, const TCHAR* StartDirectory, const TCHAR* Filename, bool Files, bool Directories);
};


/*-----------------------------------------------------------------------------
	FArchiveFileReaderGeneric
-----------------------------------------------------------------------------*/

class FArchiveFileReaderGeneric : public FArchive
{
public:
	CORE_API FArchiveFileReaderGeneric( IFileHandle* InHandle, const TCHAR* InFilename, int64 InSize, uint32 InBufferSize = PLATFORM_FILE_READER_BUFFER_SIZE, uint32 InFlags = FILEREAD_None );
	CORE_API ~FArchiveFileReaderGeneric();

	CORE_API virtual void Seek( int64 InPos ) override final;
	virtual int64 Tell() override final
	{
		return Pos;
	}
	virtual int64 TotalSize() override final
	{
		return Size;
	}
	CORE_API virtual bool Close() override final;
	CORE_API virtual void Serialize( void* V, int64 Length ) override final;
	virtual FString GetArchiveName() const override
	{
		return Filename;
	}
	CORE_API virtual void FlushCache() override final;

	CORE_API virtual bool Precache(int64 PrecacheOffset, int64 PrecacheSize) override;

protected:
	CORE_API bool InternalPrecache( int64 PrecacheOffset, int64 PrecacheSize );
	/** 
	 * Platform specific seek
	 * @param InPos - Offset from beginning of file to seek to
	 * @return false on failure 
	**/
	CORE_API virtual bool SeekLowLevel(int64 InPos);
	/** Close the file handle **/
	CORE_API virtual void CloseLowLevel();
	/** 
	 * Platform specific read
	 * @param Dest - Buffer to fill in
	 * @param CountToRead - Number of bytes to read
	 * @param OutBytesRead - Bytes actually read
	**/
	CORE_API virtual void ReadLowLevel(uint8* Dest, int64 CountToRead, int64& OutBytesRead);

	/** Returns true if the archive should suppress logging in case of error */
	bool IsSilent() const
	{
		return !!(Flags & FILEREAD_Silent);
	}

	/** Filename for debugging purposes. */
	FString Filename;
	int64 Size;
	int64 Pos;
	int64 BufferBase;
	TUniquePtr<IFileHandle> Handle;
	/**
	 * The contract for the BufferWindow and the low level pos is that if we have a BufferWindow and Pos is within it, then the LowLevel Pos is at the end of the BufferWindow
	 * If we do not have a BufferWindow, or Pos is outside of it, then LowLevel Pos is at Pos.
	 */
	TArray64<uint8> BufferArray;
	int64 BufferSize;
	uint32 Flags;
	bool bFirstReadAfterSeek;

	friend class FArchiveFileReaderGenericTest;
};


/*-----------------------------------------------------------------------------
	FArchiveFileWriterGeneric
-----------------------------------------------------------------------------*/

class FArchiveFileWriterGeneric : public FArchive
{
public:
	CORE_API FArchiveFileWriterGeneric( IFileHandle* InHandle, const TCHAR* InFilename, int64 InPos, uint32 InBufferSize = PLATFORM_FILE_WRITER_BUFFER_SIZE, uint32 InFlags = FILEWRITE_None);
	CORE_API ~FArchiveFileWriterGeneric();

	CORE_API virtual void Seek( int64 InPos ) override final;
	virtual int64 Tell() override final
	{
		return Pos;
	}
	CORE_API virtual int64 TotalSize() override;
	CORE_API virtual bool Close() override final;
	CORE_API virtual void Serialize( void* V, int64 Length ) override final;
	CORE_API virtual void Flush() override final;
	virtual FString GetArchiveName() const override
	{
		return Filename;
	}

protected:
	/**
	 * Write any internal buffer to the file handle
	 * @note Doesn't flush the handle itself, so this data may be cached by the OS and not yet written to disk!
	 * @return true if there was buffer data and it was written successfully, false if there was nothing to flush or the write failed
	 */
	CORE_API bool FlushBuffer();
	/** 
	 * Platform specific seek
	 * @param InPos - Offset from beginning of file to seek to
	 * @return false on failure 
	**/
	CORE_API virtual bool SeekLowLevel(int64 InPos);
	/** 
	 * Close the file handle
	 * @return false on failure
	**/
	CORE_API virtual bool CloseLowLevel();
	/** 
	 * Platform specific write
	 * @param Src - Buffer to write out
	 * @param CountToWrite - Number of bytes to write
	 * @return false on failure 
	**/
	CORE_API virtual bool WriteLowLevel(const uint8* Src, int64 CountToWrite);

	/**
	 * Logs I/O error
	 * It is important to not call any platform API functions after the error occurred and before 
	 * calling this functions as the system error code may get reset and will not be properly
	 * logged in this message.
	 * @param Message Brief description of what went wrong
	 */
	CORE_API void LogWriteError(const TCHAR* Message, uint32 LastError);

	/** Returns true if the archive should suppress logging in case of error */
	bool IsSilent() const
	{
		return !!(Flags & FILEWRITE_Silent);
	}

	/** Filename for debugging purposes */
	FString Filename;
	uint32 Flags;
	int64 Pos;
	TUniquePtr<IFileHandle> Handle;
	TArray64<uint8> BufferArray;
	int64 BufferSize;
	bool bLoggingError;
};
