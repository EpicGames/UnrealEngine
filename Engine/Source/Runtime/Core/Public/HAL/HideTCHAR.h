// Copyright Epic Games, Inc. All Rights Reserved.

// When third parties includes define or reference the 'system' TCHAR, it will clash with our
// typedef in Platform.h when we're compiling in PLATFORM_TCHAR_IS_UTF8CHAR mode.
//
// By including this header before any third party include that refers to TCHAR, we will force
// those headers to refer to that symbol instead.  After those includes, "HAL/AllowTCHAR.h" should
// be included, e.g.:
//
// #include "HAL/HideTCHAR.h"
// #include <tchar.h>
// #include <other_third_party_header_that_refers_to_tchar.h>
// #include "HAL/AllowTCHAR.h"
//
// Linking shouldn't be affected by this 'override' because the UE_SYSTEM_TCHAR type will mangle
// in the same way.
//
// Preprocessor pasting of the token will be affected but that is rare and will need a bespoke
// solution.
//
// If we ever need to deliberately refer to the system TCHAR rather than our Platform.h TCHAR, we
// can use the UE_SYSTEM_TCHAR type instead.

// HEADER_UNIT_SKIP - Not included directly

#ifndef UE_HIDETCHAR_INCLUDED
	#define UE_HIDETCHAR_INCLUDED
#else
	#error "HAL/HideTCHAR.h is being multiply included - an include of HAL/AllowTCHAR.h is likely missing."
#endif

#if PLATFORM_TCHAR_IS_UTF8CHAR
	// Override the TCHAR name.
	#define TCHAR UE_SYSTEM_TCHAR
#endif
