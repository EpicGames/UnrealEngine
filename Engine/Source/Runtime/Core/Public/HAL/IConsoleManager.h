// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AccessDetection.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "Delegates/IDelegateInstance.h"
#include "Delegates/Delegate.h"
#include "Features/IModularFeature.h"
#include "Templates/EnableIf.h"
#include "Internationalization/Text.h"

#if !defined(UE_WITH_CVAR_HELP_TEXT)
	#define UE_WITH_CVAR_HELP_TEXT 1
#endif

#if UE_WITH_CVAR_HELP_TEXT
	#define UE_CVAR_HELP_TEXT_PARAM(HelpText) HelpText
#else
	#define UE_CVAR_HELP_TEXT_PARAM(HelpText) nullptr
#endif

#define TRACK_CONSOLE_FIND_COUNT !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

#if DO_CHECK && (!UE_BUILD_SHIPPING) && !defined(__clang_analyzer__) // Disable even if checks in shipping are enabled.
	#define cvarCheckCode( Code )		checkCode( Code )
#else
	#define cvarCheckCode(...)
#endif

#define SETBY_ERROR ((EConsoleVariableFlags)0x01000000)

template <class T> class TConsoleVariableData;

/**
 * Console variable usage guide:
 *
 * The variable should be creates early in the initialization but not before (not in global variable construction).
 * Choose the right variable type, consider using a console command if more functionality is needed (see Exec()).
 * Available types: bool, int, float, bool&, int&, float&, string
 * Always provide a good help text, other should be able to understand the function of the console variable by reading this help.
 * The help length should be limited to a reasonable width in order to work well for low res screen resolutions.
 *
 * Usage in the game console:
 *   <COMMAND> ?				print the HELP
 *   <COMMAND>	 				print the current state of the console variable
 *   <COMMAND> x 				set and print the new state of the console variable
 *
 * All variables support auto completion. The single line help that can show up there is currently not connected to the help as the help text
 * is expected to be multi line.
 * The former Exec() system can be used to access the console variables.
 * Use console variables only in main thread.
 * The state of console variables is not network synchronized or serialized (load/save). The plan is to allow to set the state in external files (game/platform/engine/local).
 */

/**
 * Bitmask 0x1, 0x2, 0x4, ..
 */
enum EConsoleVariableFlags
{
	/* Mask for flags. Use this instead of ~ECVF_SetByMask */
	ECVF_FlagMask = 0x0000ffff,

	/**
	 * Default, no flags are set, the value is set by the constructor 
	 */
	ECVF_Default = 0x0,
	/**
	 * Console variables marked with this flag behave differently in a final release build.
	 * Then they are are hidden in the console and cannot be changed by the user.
	 */
	ECVF_Cheat = 0x1,
	/**
	 * Console variables cannot be changed by the user (from console).
	 * Changing from C++ or ini is still possible.
	 */
	ECVF_ReadOnly = 0x4,
	/**
	 * UnregisterConsoleObject() was called on this one.
	 * If the variable is registered again with the same type this object is reactivated. This is good for DLL unloading.
	 */
	ECVF_Unregistered = 0x8,
	/**
	 * This flag is set by the ini loading code when the variable wasn't registered yet.
	 * Once the variable is registered later the value is copied over and the variable is destructed.
	 */
	ECVF_CreatedFromIni = 0x10,
	/**
	 * Maintains another shadow copy and updates the copy with render thread commands to maintain proper ordering.
	 * Could be extended for more/other thread.
 	 * Note: On console variable references it assumes the reference is accessed on the render thread only
	 * (Don't use in any other thread or better don't use references to avoid the potential pitfall).
	 */
	ECVF_RenderThreadSafe = 0x20,

	/* ApplyCVarSettingsGroupFromIni will complain if this wasn't set, should not be combined with ECVF_Cheat and ECVF_ExcludeFromPreview. 
	 * They are automatically added as  ECVF_Preview unless ECVF_ExcludeFromPreview is used
	 */
	ECVF_Scalability = 0x40,

	/* those cvars control other cvars with the flag ECVF_Scalability, names should start with "sg." */
	ECVF_ScalabilityGroup = 0x80,

	/* CVars with this flag will be included in the device profile previews. */
	ECVF_Preview = 0x100,

	/* Cvars with this flag will modify the Shader Keystring for All Platforms*/
	ECVF_GeneralShaderChange = 0x200,

	/* Cvars with this flag will modify the Shader Keystring for Mobile Platforms*/
	ECVF_MobileShaderChange = 0x400,

	/* Cvars with this flag will modify the Shader Keystring for Desktop Platforms*/
	ECVF_DesktopShaderChange = 0x800,

	/* CVars with this flag will be excluded from the device profile previews. */
	ECVF_ExcludeFromPreview = 0x1000,

	/**
	 * CVars with this flag will be saved into local temp file for next boot after hotfixed. Normally use for feature switch which is too early, even before launching the hotfix http request
	 * Use it carefully with these constrains in mind: 
	 *  - WARNING: This exposes the config to end user who could change it through the local temp file in shipping client, it's your responsibility to make sure client can't "legally" cheat through this type of CVar;
	 *  - For the first time get hotfixed, the value will change from default value during boot to the hotfixed value after the hotfix http request, so make sure caching it when boot if the logic doesn't support to change at runtime;
	 *  - When there is a new released game version, the saved local temp file of last version could still be there. So it will still got read once. Keep the logic work well with the old value, or remove the CVar if the old value no longer supported;
	 *  - It will only work for game client, not for dedicated servers which will be dynamically allocated on different server instances.
	 */
	ECVF_SaveForNextBoot = 0x2000,

	// ------------------------------------------------

	/* Set flags */
	ECVF_SetFlagMask =				0x00ff0000,

	// Use to set a cvar without calling all cvar sinks. Much faster, but potentially unsafe. Use only if you know the particular cvar/setting does not require a sink call
	ECVF_Set_NoSinkCall_Unsafe =	0x00010000,
	
	// Similar to ECVF_Set_NoSinkCall_Unsafe, but this will do nothing but set the cvar and flags, and push updates to render thread, no sinks, etc
	ECVF_Set_SetOnly_Unsafe = 		0x00020000,

	// When this is set, this will stomp on existing tagged array type values in place, instead of adding to the end of the array
	ECVF_Set_ReplaceExistingTag =	0x00040000,

	// ------------------------------------------------

	/* to get some history of where the last value was set by ( useful for track down why a cvar is in a specific state */
	ECVF_SetByMask =				0xff000000,

	// The ECVF_SetBy flags are sorted in override order (weak to strong), the value is not serialized. It only affects the override behavior when calling Set()

	// Lowest priority (default after console variable creation)
	ECVF_SetByConstructor =			0x00000000,
	// Reserved value for SETBY_ERROR. Please don't use.
	// SETBY_ERROR =				0x01000000 
	// Set by scalability groups from Scalability.ini (lower priority than game settings so it's easier to override partially). THIS IS AN ARRAY TYPE
	ECVF_SetByScalability =			0x02000000,
	// Default priority for engine-level game user settings, platform-specific settings will override this
	ECVF_SetByGameSetting =			0x03000000,
	// Set by project settings UI or specific sections in ini file (higher priority than game setting to allow enforcing some settings for this project)
	ECVF_SetByProjectSetting =		0x04000000,
	// Used by the [ConsoleVariables] section of Engine.ini as well as FSystemSettings
	ECVF_SetBySystemSettingsIni =	0x05000000,
	// Dyanmically loaded/unloaded plugins, used with the History concept to restore cvars on plugin unload. THIS IS AN ARRAY TYPE meaning multiple vales with this SetBy are stored in the history.
	ECVF_SetByPluginLowPriority =	0x06000000,
	// Per device settings using the DeviceProfiles.ini hierarchy (e.g. specific iOS device, higher priority than per project to do device specific settings). THIS IS AN ARRAY TYPE
	ECVF_SetByDeviceProfile =		0x07000000,
	// Dyanmically loaded/unloaded plugins, used with the History concept to restore cvars on plugin unload. THIS IS AN ARRAY TYPE meaning multiple vales with this SetBy are stored in the history.
	ECVF_SetByPluginHighPriority =	0x08000000,
	// User settable game overrides, used for GameUserSettings fields that need to override device specific settings
	ECVF_SetByGameOverride =		0x09000000,
	// Set by local consolevariables.ini, mostly used for testing multiple projects
	ECVF_SetByConsoleVariablesIni = 0x0A000000,
	// Set by hotfix. THIS IS AN ARRAY TYPE meaning multiple vales with this SetBy are stored in the history
	ECVF_SetByHotfix =				0x0B000000,
	// Set for previewing in editor. THIS IS AN ARRAY TYPE meaning multiple vales with this SetBy are stored in the history
	ECVF_SetByPreview =				0x0C000000,
	// Used by some command line parameters, others use the Console priority instead
	ECVF_SetByCommandline =			0x0D000000,
	// Used for high priority temporary debugging or operation modes
	ECVF_SetByCode =				0x0E000000,
	// Used for temporarily replacing a variable.  THIS IS AN ARRAY TYPE meaning multiple vales with this SetBy are stored in the history
	ECVF_SetByTemp =                0x0F000000,
	// Highest priority used via editor UI or or game/editor interactive console
	ECVF_SetByConsole =				0x10000000,


	// ------------------------------------------------
};


#define ENUMERATE_SET_BY(op) \
	op(Constructor) \
	op(Scalability) \
	op(GameSetting) \
	op(ProjectSetting) \
	op(SystemSettingsIni) \
	op(PluginLowPriority) \
	op(DeviceProfile) \
	op(PluginHighPriority) \
	op(GameOverride) \
	op(ConsoleVariablesIni) \
	op(Hotfix) \
	op(Preview) \
	op(Commandline) \
	op(Code) \
	op(Temp) \
	op(Console)


/** Returns human readable ECVF_SetByMask bits of the console variable flags. */
extern CORE_API const TCHAR* GetConsoleVariableSetByName(EConsoleVariableFlags ConsoleVariableFlags);

/** Inverse of GetConsoleVariableSetByName() */
extern CORE_API EConsoleVariableFlags GetConsoleVariableSetByValue(const TCHAR* SetByName);

namespace ConsoleVariablePlatformMapping
{
	// Store a mapping from EShaderPlatform (as int) to a Platform that is currently cooking for that ShaderPlatform
	CORE_API void RegisterShaderPlatformToPlatformMapping(int ShaderPlatform, FName PlatformName);
	// Store a mapping from Platform to a DevicveProfile that is currently being used when cooking for that Platform
	CORE_API void RegisterPlatformToDeviceProfileMapping(FName PlatformName, FName DeviceProfileName);

	// Receives the mapping previously set with RegisterShaderPlatformToPlatformMapping
	// Returns NAME_None if no mapping exists
	CORE_API FName GetShaderPlatformToPlatformMapping(int ShaderPlatform);
	// Receives the mapping previously set with RegisterPlatformToDeviceProfileMapping
	// Returns NAME_None if no mapping exists
	CORE_API FName GetPlatformToDeviceProfileMapping(FName PlatformName);
}

class IConsoleVariable;

#if !NO_CVARS

/** Console variable delegate type  This is a void callback function. */
DECLARE_DELEGATE_OneParam( FConsoleVariableDelegate, IConsoleVariable* );

/** Console variable multicast delegate type. */
DECLARE_MULTICAST_DELEGATE_OneParam(FConsoleVariableMulticastDelegate, IConsoleVariable*);

/** Console object with name multicast delegate type. */
DECLARE_MULTICAST_DELEGATE_TwoParams(FConsoleObjectWithNameMulticastDelegate, const TCHAR*, IConsoleObject*);

/** Console command delegate type (takes no arguments.)  This is a void callback function. */
DECLARE_DELEGATE( FConsoleCommandDelegate );

/** Console command delegate type (with arguments.)  This is a void callback function that always takes a list of arguments. */
DECLARE_DELEGATE_OneParam( FConsoleCommandWithArgsDelegate, const TArray< FString >& );

/** Console command delegate type with a world argument. This is a void callback function that always takes a world. */
DECLARE_DELEGATE_OneParam( FConsoleCommandWithWorldDelegate, UWorld* );

/** Console command delegate type (with a world and arguments.)  This is a void callback function that always takes a list of arguments and a world. */
DECLARE_DELEGATE_TwoParams(FConsoleCommandWithWorldAndArgsDelegate, const TArray< FString >&, UWorld*);

/** Console command delegate type (with arguments and output device.)  This is a void callback function that always takes a list of arguments and output device. */
DECLARE_DELEGATE_TwoParams(FConsoleCommandWithArgsAndOutputDeviceDelegate, const TArray< FString >&, FOutputDevice&);

/** Console command delegate type (with a world arguments and output device.)  This is a void callback function that always takes a list of arguments, a world and output device. */
DECLARE_DELEGATE_ThreeParams(FConsoleCommandWithWorldArgsAndOutputDeviceDelegate, const TArray< FString >&, UWorld*, FOutputDevice&);

/** Console command delegate type with the output device passed through. */
DECLARE_DELEGATE_OneParam( FConsoleCommandWithOutputDeviceDelegate, FOutputDevice& );

#else

template <typename DerivedType, typename... ParamTypes>
struct FNullConsoleVariableDelegate
{
	/**
	 * Static: Creates a raw C++ pointer global function delegate
	 */
	template <typename... VarTypes>
	inline static DerivedType CreateStatic(typename TIdentity<void (*)(ParamTypes..., VarTypes...)>::Type, VarTypes...)
	{
		return {};
	}

	template<typename FunctorType, typename... VarTypes>
	inline static DerivedType CreateLambda(FunctorType&&, VarTypes...)
	{
		return {};
	}

	template<typename UserClass, typename FunctorType, typename... VarTypes>
	inline static DerivedType CreateSPLambda(UserClass*, FunctorType&&, VarTypes...)
	{
		return {};
	}

	template<typename UserClass, typename FunctorType, typename... VarTypes>
	inline static DerivedType CreateWeakLambda(UserClass*, FunctorType&&, VarTypes...)
	{
		return {};
	}

	template <typename UserClass, typename... VarTypes>
	inline static DerivedType CreateRaw(UserClass*, typename TMemFunPtrType<false, UserClass, void (ParamTypes..., VarTypes...)>::Type, VarTypes...)
	{
		return {};
	}
	template <typename UserClass, typename... VarTypes>
	inline static DerivedType CreateRaw(UserClass*, typename TMemFunPtrType<true, UserClass, void (ParamTypes..., VarTypes...)>::Type, VarTypes...)
	{
		return {};
	}

	template <typename UserClass, typename... VarTypes>
	inline static DerivedType CreateSP(const TSharedRef<UserClass>&, typename TMemFunPtrType<false, UserClass, void (ParamTypes..., VarTypes...)>::Type, VarTypes...)
	{
		return {};
	}
	template <typename UserClass, typename... VarTypes>
	inline static DerivedType CreateSP(const TSharedRef<UserClass>&, typename TMemFunPtrType<true, UserClass, void (ParamTypes..., VarTypes...)>::Type, VarTypes...)
	{
		return {};
	}

	template <typename UserClass, typename... VarTypes>
	inline static DerivedType CreateSP(UserClass*, typename TMemFunPtrType<false, UserClass, void (ParamTypes..., VarTypes...)>::Type, VarTypes...)
	{
		return {};
	}
	template <typename UserClass, typename... VarTypes>
	inline static DerivedType CreateSP(UserClass*, typename TMemFunPtrType<true, UserClass, void (ParamTypes..., VarTypes...)>::Type, VarTypes...)
	{
		return {};
	}
	
	template <typename UserClass, typename... VarTypes>
	inline static DerivedType CreateThreadSafeSP(const TSharedRef<UserClass, ESPMode::ThreadSafe>&, typename TMemFunPtrType<false, UserClass, void (ParamTypes..., VarTypes...)>::Type, VarTypes...)
	{
		return {};
	}
	template <typename UserClass, typename... VarTypes>
	inline static DerivedType CreateThreadSafeSP(const TSharedRef<UserClass, ESPMode::ThreadSafe>&, typename TMemFunPtrType<true, UserClass, void (ParamTypes..., VarTypes...)>::Type, VarTypes...)
	{
		return {};
	}

	template <typename UserClass, typename... VarTypes>
	inline static DerivedType CreateThreadSafeSP(UserClass*, typename TMemFunPtrType<false, UserClass, void (ParamTypes..., VarTypes...)>::Type, VarTypes...)
	{
		return {};
	}
	template <typename UserClass, typename... VarTypes>
	inline static DerivedType CreateThreadSafeSP(UserClass*, typename TMemFunPtrType<true, UserClass, void (ParamTypes..., VarTypes...)>::Type, VarTypes...)
	{
		return {};
	}

	template <typename UObjectTemplate, typename... VarTypes>
	inline static DerivedType CreateUFunction(UObjectTemplate*, const FName&, VarTypes...)
	{
		return {};
	}

	template <typename UserClass, typename... VarTypes>
	inline static DerivedType CreateUObject(UserClass*, typename TMemFunPtrType<false, UserClass, void (ParamTypes..., VarTypes...)>::Type, VarTypes...)
	{
		return {};
	}
	template <typename UserClass, typename... VarTypes>
	inline static DerivedType CreateUObject(UserClass*, typename TMemFunPtrType<true, UserClass, void (ParamTypes..., VarTypes...)>::Type, VarTypes...)
	{
		return {};
	}

	FDelegateHandle GetHandle() const
	{
		return {};
	}

	bool ExecuteIfBound(ParamTypes...)
	{
		return false;
	}
};

struct FConsoleVariableDelegate								: FNullConsoleVariableDelegate<FConsoleVariableDelegate, IConsoleVariable*> {};
struct FConsoleCommandDelegate								: FNullConsoleVariableDelegate<FConsoleCommandDelegate> {};
struct FConsoleCommandWithArgsDelegate						: FNullConsoleVariableDelegate<FConsoleCommandWithArgsDelegate, const TArray<FString>&> {};
struct FConsoleCommandWithWorldDelegate						: FNullConsoleVariableDelegate<FConsoleCommandWithWorldDelegate, UWorld*> {};
struct FConsoleCommandWithWorldAndArgsDelegate				: FNullConsoleVariableDelegate<FConsoleCommandWithWorldAndArgsDelegate, const TArray<FString>&, UWorld*> {};
struct FConsoleCommandWithArgsAndOutputDeviceDelegate		: FNullConsoleVariableDelegate<FConsoleCommandWithArgsAndOutputDeviceDelegate, const TArray<FString>&, FOutputDevice&> {};
struct FConsoleCommandWithWorldArgsAndOutputDeviceDelegate	: FNullConsoleVariableDelegate<FConsoleCommandWithWorldArgsAndOutputDeviceDelegate, const TArray<FString>&, UWorld*, FOutputDevice&> {};
struct FConsoleCommandWithOutputDeviceDelegate				: FNullConsoleVariableDelegate<FConsoleCommandWithOutputDeviceDelegate, FOutputDevice&> {};

#endif

template <class T> class TConsoleVariableData;

/**
 * Interface for console objects (variables and commands)
 */
class IConsoleObject
{

public:

	IConsoleObject()
#if TRACK_CONSOLE_FIND_COUNT
		: FindCallCount(0)
#endif
	{}

	virtual ~IConsoleObject() = default;

#if UE_WITH_CVAR_HELP_TEXT
	/**
	 *  @return never 0, can be multi line ('\n')
	 */
	virtual const TCHAR* GetHelp() const = 0;
	/**
	 *  @return never 0, can be multi line ('\n')
	 */
	virtual void SetHelp(const TCHAR* Value) = 0;
	/**
	 *  @return a (potentially) more detailed help than GetHelp (e.g. current value for console variables)
	 */
	virtual FText GetDetailedHelp() const
	{
		// By default, just return the standard help message :
		return FText::FromString(GetHelp());
	}
#else
	const TCHAR* GetHelp() const { return TEXT(""); }
	void SetHelp(const TCHAR*) {}
	FText GetDetailedHelp() const { return FText::FromString(TEXT("")); }
#endif

	/**
	 * Get the internal state of the flags.
	 */
	virtual EConsoleVariableFlags GetFlags() const = 0;
	/**
	 * Sets the internal flag state to the specified value.
	 */
	virtual void SetFlags(const EConsoleVariableFlags Value) = 0;

	// Convenience methods -------------------------------------

	/**
	 * Removes the specified flags in the internal state.
	 */
	void ClearFlags(const EConsoleVariableFlags Value)
	{
		uint32 New = (uint32)GetFlags() & ~(uint32)Value;
	
		SetFlags((EConsoleVariableFlags)New);
	}
	/**
	 * Test is any of the specified flags is set in the internal state.
	 */
	bool TestFlags(const EConsoleVariableFlags Value) const
	{
		return ((uint32)GetFlags() & (uint32)Value) != 0;
	}

	/**
	 * Test the validity of the variable wrt its flags and current build type (e.g. ECVF_Cheat variables are disabled on some targets)
	 */
	bool IsEnabled() const
	{
#if DISABLE_CHEAT_CVARS
		if (TestFlags(ECVF_Cheat))
		{
			return false;
		}
#endif // DISABLE_CHEAT_CVARS
		if (TestFlags(ECVF_Unregistered))
		{
			return false;
		}
		return true;
	}


	/**
	 * If the object has a parent (for instance the main cvar that owns an other-platform cvar), return it
	 */
	virtual IConsoleObject* GetParentObject() const
	{
		return nullptr;
	}
	
	/**
	 * Casts this object to an IConsoleVariable, returns 0 if it's not
	 */
	virtual class IConsoleVariable* AsVariable()
	{
		return 0; 
	}

	virtual bool IsVariableBool() const { return false; }
	virtual bool IsVariableInt() const { return false; }
	virtual bool IsVariableFloat() const { return false; }
	virtual bool IsVariableString() const { return false; }

	virtual class TConsoleVariableData<bool>* AsVariableBool()
	{
		ensureMsgf(false, TEXT("Attempted to access variable data of a console variable type that doesn't support it.  For example FindTConsoleVariableData* on a FAutoConsoleVariableRef."));
		return 0;
	}

	virtual class TConsoleVariableData<int32>* AsVariableInt()
	{
		ensureMsgf(false, TEXT("Attempted to access variable data of a console variable type that doesn't support it.  For example FindTConsoleVariableData* on a FAutoConsoleVariableRef."));
		return 0; 
	}

	virtual class TConsoleVariableData<float>* AsVariableFloat()
	{
		ensureMsgf(false, TEXT("Attempted to access variable data of a console variable type that doesn't support it.  For example FindTConsoleVariableData* on a FAutoConsoleVariableRef."));
		return 0; 
	}

	virtual class TConsoleVariableData<FString>* AsVariableString()
	{
		ensureMsgf(false, TEXT("Attempted to access variable data of a console variable type that doesn't support it.  For example FindTConsoleVariableData* on a FAutoConsoleVariableRef."));
		return 0;
	}

	/**
	 * Casts this object to an IConsoleCommand, verifying first that it is safe to do so
	 */
	virtual struct IConsoleCommand* AsCommand()
	{
		return 0; 
	}

	/**
	 * A shadow ConsoleObject is one that exists and can be used, but shouldn't be iterated over in anything that walks over every 
	 * object and acts on it
	 */
	virtual bool IsShadowObject() const
	{
		return false;
	}

	/**
	 * If IsShadowObject() returns true, this function will return the "real" variable the shadow cvar is shadowing. Generally not needed,
	 * as most functions (like GetInt, Set, etc) will operate on the real variable already
	 */
	virtual IConsoleObject* GetShadowedObject() const
	{
		return nullptr;
	}

	virtual bool IsDeprecated() const
	{
		return false;
	}

private: // -----------------------------------------

#if TRACK_CONSOLE_FIND_COUNT
	// no longer pure visual, if that causes problems we can change the interface
	// to track down FindConsoleObject/FindConsoleVariable calls without static
	uint32 FindCallCount;
#endif

	/**
	 *  should only be called by the manager, needs to be implemented for each instance
	 */
	virtual void Release() = 0;

	friend class FConsoleManager;
};

/**
 * Interface for console variables
 */
class IConsoleVariable : public IConsoleObject
{
public:

	struct FResolvedContext
	{
		EConsoleVariableFlags Flags;
		FName Tag;
	};

	struct FSetContext
	{
		FSetContext()
		{
		}

		FSetContext(EConsoleVariableFlags InFlags, FName InTag)
			: Flags(InFlags)
			, Tag(InTag)
		{
		}


		enum class EPriorityMode : uint8
		{
			Set,
			// the following are limited by Max priority if its set
			ReplaceCurrent,
		};

		enum class ETagMode : uint8
		{
			// Sets the Tag at the end of the array. Remove the Tag if it already exists.
			Set,
			// Replace the Tag in-place if it already exists. Sets the Tag at the end if it doesn't exist.
			Replace,
			// Replaces the current highest Tag.
			ReplaceCurrent,
		};


		EConsoleVariableFlags Flags = ECVF_SetByCode;
		EPriorityMode PriorityMode = EPriorityMode::Set;

		FName Tag = NAME_None;
		ETagMode TagMode = ETagMode::Set;

		EConsoleVariableFlags MaxPriority = ECVF_SetByConsole;
		EConsoleVariableFlags MinPriority = SETBY_ERROR;
	};

	virtual FResolvedContext ResolveContext(const FSetContext& Context) = 0;

	/**
	 * Set the internal value from the specified string. 
	 * @param InValue Internal Value to Set
	 * @param Context The ResolvedContext of the FSetContext
	 **/
	virtual void Set(const TCHAR* InValue, const FResolvedContext& Context) = 0;

	/**
	 * Unsets the value at a certain SetBy priority (this is only useful when UE_ALLOW_CVAR_HISTORY is set). The value of the CVar
	 * will be recalculated based on remaining History levels
	 * @param SetBy anything in ECVF_LastSetMask e.g. ECVF_SetByScalability
	 * @param Tag tag used to remove a setting from an ARRAY type SetBy
	 **/
	virtual void Unset(EConsoleVariableFlags SetBy, FName Tag = NAME_None) = 0;

	/**
	 * Get the internal value as a bool, works on bools, ints and floats.
	 */
	virtual bool GetBool() const = 0;
	/**
	 * Get the internal value as int (should not be used on strings).
	 * @return value is not rounded (simple cast)
	 */
	virtual int32 GetInt() const = 0;
	/** Get the internal value as float (works on all types). */
	virtual float GetFloat() const = 0;
	/** Get the internal value as string (works on all types). */
	virtual FString GetString() const = 0;

	/** Generic versions for templated code */
	void GetValue(int32& OutIntValue) const
	{
		OutIntValue = GetInt();
	}
	void GetValue(bool& OutBoolValue) const
	{
		OutBoolValue = GetBool();
	}
	void GetValue(float& OutFloatValue) const
	{
		OutFloatValue = GetFloat();
	}
	void GetValue(FString& OutStringValue) const
	{
		OutStringValue = GetString();
	}

	/**
	 * Allows to specify a callback function that is called when the console variable value changes.
 	 * Is even called if the value is the same as the value before. Will always be on the game thread.
	 * This can be dangerous (instead try to use RegisterConsoleVariableSink())
	 * - Setting other console variables in the delegate can cause infinite loops
	 * - Setting many console variables could result in wasteful cycles (e.g. if multiple console variables require to reattach all objects it would happen for each one)
	 * - The call can be at any time during initialization.
	 * As this cannot be specified during constructions you are not called on creation.
	 * We also don't call for the SetOnChangedCallback() call as this is up to the caller.
	 **/
	virtual void SetOnChangedCallback(const FConsoleVariableDelegate& Callback) = 0;

	/**
	 * Remove the callback function set to this console variable.
	 */
	virtual void ClearOnChangedCallback(){}

	virtual FConsoleVariableMulticastDelegate& OnChangedDelegate() = 0;

	/**
	 * Get the value this CVar was constructed with
	 */
	virtual FString GetDefaultValue() = 0;

#if ALLOW_OTHER_PLATFORM_CONFIG

	/**
	 * Get a CVar opject that matches this cvar, but contains the value of the platform given. This will trigger a lof of all cvars for this platform/DP if it doesn't exist
	 * Note: If this causes a compile error due to pure virtual, make sure your IConsoleVariable subclass inherits from FConsoleVariableExtendedData
	 */
	virtual TSharedPtr<IConsoleVariable> GetPlatformValueVariable(FName PlatformName, const FString& DeviceProfileName=FString()) = 0;

	/**
	 * Checks if the CVar has a cached value for the given platform
	 */
	virtual bool HasPlatformValueVariable(FName PlatformName, const FString& DeviceProfileName=FString()) = 0;

	/**
	 * Used only for debugging/iterating, this will clear all of the other platform's cvar objects, which will 
	 * force a fresh lookup (with fresh ini files, likely) on next call to GetPlatformValueVariable
	 */
	virtual void ClearPlatformVariables(FName PlatformName=NAME_None)
	{
	}
	
#endif

	/**
	 * Print the history to a log
	*/
	virtual void LogHistory(FOutputDevice& Ar) = 0;
	
	/**
	 * Track memory used by history data
	 */
	virtual SIZE_T GetHistorySize() = 0;

	// convenience methods

	/** Set the internal value from the SetContext. */
	void Set(const TCHAR* InValue, const FSetContext& Context)
	{
		Set(InValue, this->ResolveContext(Context));
	}

	/** Set the internal value from the specified bool. */
	void Set(bool InValue, const FSetContext& Context)
	{
		// NOTE: Bool needs to use 1 and 0 here rather than true/false, as this may be a int32 or something
		// and eventually this code calls, TTypeFromString<T>::FromString which won't handle the true/false,
		// but 1 and 0 will work for whatever.
		// inefficient but no common code path
		Set(InValue ? TEXT("1") : TEXT("0"), Context);
	}
	/** Set the internal value from the specified int. */
	void Set(int32 InValue, const FSetContext& Context)
	{
		// inefficient but no common code path
		Set(*FString::Printf(TEXT("%d"), InValue), Context);
	}
	/** Set the internal value from the specified float. */
	void Set(float InValue, const FSetContext& Context)
	{
		// inefficient but no common code path
		Set(*FString::Printf(TEXT("%g"), InValue), Context);
	}


	template <typename T>
	void Set(T Value, EConsoleVariableFlags Flags = ECVF_SetByCode, FName Tag = NAME_None)
	{
		Set(Value, FSetContext(Flags, Tag));
	}

	template <typename T>
	void SetWithCurrentPriority(
		T Value, 
		FName Tag = NAME_None, 
		EConsoleVariableFlags MaxPriority = ECVF_SetByConsole,
		EConsoleVariableFlags MinPriority = SETBY_ERROR)
	{
		checkf(MinPriority <= MaxPriority, TEXT("MinPriority must be <= MaxPriority"))
		FSetContext Context;
		Context.Tag = Tag;
		Context.PriorityMode = FSetContext::EPriorityMode::ReplaceCurrent;
		Context.MinPriority = MinPriority;
		Context.MaxPriority = MaxPriority;
		EConsoleVariableFlags CurFlags = (EConsoleVariableFlags)(GetFlags() & ECVF_SetByMask);
		Context.Flags = CurFlags;
		Set(Value, Context);
	}

	template<typename T>
	void ReplaceCurrentPriorityAndTag(
		T InValue, 
		EConsoleVariableFlags MaxPriority = ECVF_SetByConsole,
		EConsoleVariableFlags MinPriority = SETBY_ERROR)
	{
		checkf(MinPriority <= MaxPriority, TEXT("MinPriority must be <= MaxPriority"))
		FSetContext Context;
		Context.TagMode = FSetContext::ETagMode::ReplaceCurrent;
		Context.PriorityMode = FSetContext::EPriorityMode::ReplaceCurrent;
		Context.MinPriority = MinPriority;
		Context.MaxPriority = MaxPriority;
		EConsoleVariableFlags CurFlags = (EConsoleVariableFlags)(GetFlags() & ECVF_SetByMask);
		Context.Flags = CurFlags;
		Set(InValue, Context);
	}


};

/**
 * Interface for console commands
 */
struct IConsoleCommand : public IConsoleObject
{
	/**
	 * Executes this command (optionally, with arguments)
	 *
	 * @param	Args		Argument list for this command
	 * @param	InWorld		World context for this command
	 * @return	True if the delegate for this command was executed successfully
	 */
	virtual bool Execute( const TArray< FString >& Args, UWorld* InWorld, class FOutputDevice& OutputDevice ) = 0;
};

/**
 * Interface to propagate changes of console variables to another thread
 */
struct IConsoleThreadPropagation
{
	virtual void OnCVarChange(int32& Dest, int32 NewValue) = 0;
	virtual void OnCVarChange(float& Dest, float NewValue) = 0;
	virtual void OnCVarChange(bool& Dest, bool NewValue) = 0;
	virtual void OnCVarChange(FString& Dest, const FString& NewValue) = 0;
	virtual void OnCVarChange(FName& Dest, const FName& NewValue) = 0;
};

/**
 * Declares a delegate type that's used by the console manager to call back into a user function for each
 * known console object.
 *
 * First parameter is the Name string for the current console object
 * Second parameter is the current console object
 */
DECLARE_DELEGATE_TwoParams( FConsoleObjectVisitor, const TCHAR*, IConsoleObject* );


/**
 * Class representing an handle to an online delegate.
 */
class FConsoleVariableSinkHandle
{
public:
	FConsoleVariableSinkHandle()
	{
	}

	explicit FConsoleVariableSinkHandle(FDelegateHandle InHandle)
		: Handle(InHandle)
	{
	}

	template <typename MulticastDelegateType>
	void RemoveFromDelegate(MulticastDelegateType& MulticastDelegate)
	{
		MulticastDelegate.Remove(Handle);
	}

	template <typename DelegateType>
	bool HasSameHandle(const DelegateType& Delegate) const
	{
		return Delegate.GetHandle() == Handle;
	}

private:
	FDelegateHandle Handle;
};

struct FConsoleSuggestion
{
	FString Name;
	FString Help;

	FConsoleSuggestion(FString InName, FString InHelp)
	: Name(InName)
	, Help(InHelp)
	{}

	FConsoleSuggestion()
	{}
};

/**
 * Handles executing console commands
 */
class IConsoleCommandExecutor : public IModularFeature
{
public:
	virtual ~IConsoleCommandExecutor() = default;

	/**
	 * Get the name identifying this modular feature set.
	 */
	static FName ModularFeatureName()
	{
		static const FName Name = "ConsoleCommandExecutor";
		return Name;
	}

	/**
	 * Get the name of this executor.
	 */
	virtual FName GetName() const = 0;

	/**
	 * Get the display name of this executor.
	 */
	virtual FText GetDisplayName() const = 0;

	/**
	 * Get the description of this executor.
	 */
	virtual FText GetDescription() const = 0;

	/**
	 * Get the hint text of this executor.
	 */
	virtual FText GetHintText() const = 0;

	/**
	* Get the list of auto-complete suggestions for the given command.
	*/
	virtual void GetSuggestedCompletions(const TCHAR* Input, TArray<FConsoleSuggestion>& Out) = 0;

	/**
	 * Get the list of commands that this executor has recently processed.
	 */
	virtual void GetExecHistory(TArray<FString>& Out) = 0;

	/**
	 * Execute the given command using this executor.
	 * @return true if the command was recognized.
	 */
	virtual bool Exec(const TCHAR* Input) = 0;

	/**
	 * True if we allow the console to be closed using the "open console" hot-key.
	 * @note Some scripting languages use the default "open console" hot-key (~) in their code, so these should return false.
	 */
	virtual bool AllowHotKeyClose() const = 0;

	/**
	 * True if we allow the console to create multi-line commands.
	 */
	virtual bool AllowMultiLine() const = 0;

	/**
	* Returns the hotkey for this executor
	*/
	virtual struct FInputChord GetHotKey() const = 0;
	/**
	* Returns the hotkey to switch to the next executor. This is usually the same hotkey as GetHotKey but with a modifier,
	* such as "`" turning into "Ctrl + `".
	*/
	virtual struct FInputChord GetIterateExecutorHotKey() const = 0;
};


/**
 * handles console commands and variables, registered console variables are released on destruction
 */
struct IConsoleManager
{
	/**
	 * Create a bool console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	virtual IConsoleVariable* RegisterConsoleVariable(const TCHAR* Name, bool DefaultValue, const TCHAR* Help, uint32 Flags = ECVF_Default) = 0;
	/**
	 * Create a int console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	virtual IConsoleVariable* RegisterConsoleVariable(const TCHAR* Name, int32 DefaultValue, const TCHAR* Help, uint32 Flags = ECVF_Default) = 0;
	/**
	 * Create a float console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	virtual IConsoleVariable* RegisterConsoleVariable(const TCHAR* Name, float DefaultValue, const TCHAR* Help, uint32 Flags = ECVF_Default) = 0;
	/**
	 * Create a string console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	virtual IConsoleVariable* RegisterConsoleVariable(const TCHAR* Name, const TCHAR* DefaultValue, const TCHAR* Help, uint32 Flags = ECVF_Default) = 0;
	/**
	 * Create a string console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	virtual IConsoleVariable* RegisterConsoleVariable(const TCHAR* Name, const FString& DefaultValue, const TCHAR* Help, uint32 Flags = ECVF_Default) = 0;
	/**
	 * Create a reference to a bool console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	virtual IConsoleVariable* RegisterConsoleVariableRef(const TCHAR* Name, bool& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default) = 0;
	/**
	 * Create a reference to a int console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	virtual IConsoleVariable* RegisterConsoleVariableRef(const TCHAR* Name, int32& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default) = 0;
	/**
	 * Create a reference to a float console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	virtual IConsoleVariable* RegisterConsoleVariableRef(const TCHAR* Name, float& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default) = 0;
	/**
	* Create a reference to a string console variable
	* @param Name must not be 0
	* @param Help must not be 0
	* @param Flags bitmask combined from EConsoleVariableFlags
	*/
	virtual IConsoleVariable* RegisterConsoleVariableRef(const TCHAR* Name, FString& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default) = 0;
	/**
	* Create a reference to a name console variable
	* @param Name must not be 0
	* @param Help must not be 0
	* @param Flags bitmask combined from EConsoleVariableFlags
	*/
	virtual IConsoleVariable* RegisterConsoleVariableRef(const TCHAR* Name, FName& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default) = 0;

	/**
	* Create a console variable that delegates get/set to external functions.
	* The Getter is called on every read; the Setter is called on every write.
	* ECVF_RenderThreadSafe is NOT supported with delegate variables.
	*
	* @param Name must not be 0, must not be empty, must not contain spaces
	* @param DefaultValue initial value passed to Setter during registration
	* @param Getter called to read the current value
	* @param Setter called to write a new value
	* @param Help must not be 0, must not be empty
	* @param Flags bitmask combined from EConsoleVariableFlags
	*/
	virtual IConsoleVariable* RegisterDelegatedConsoleVariable(
		const TCHAR* Name, int32 DefaultValue,
		TFunction<int32()> Getter, TFunction<void(const int32&)> Setter,
		const TCHAR* Help, uint32 Flags = ECVF_Default) = 0;

	virtual IConsoleVariable* RegisterDelegatedConsoleVariable(
		const TCHAR* Name, float DefaultValue,
		TFunction<float()> Getter, TFunction<void(const float&)> Setter,
		const TCHAR* Help, uint32 Flags = ECVF_Default) = 0;

	virtual IConsoleVariable* RegisterDelegatedConsoleVariable(
		const TCHAR* Name, bool DefaultValue,
		TFunction<bool()> Getter, TFunction<void(const bool&)> Setter,
		const TCHAR* Help, uint32 Flags = ECVF_Default) = 0;

	virtual IConsoleVariable* RegisterDelegatedConsoleVariable(
		const TCHAR* Name, const FString& DefaultValue,
		TFunction<FString()> Getter, TFunction<void(const FString&)> Setter,
		const TCHAR* Help, uint32 Flags = ECVF_Default) = 0;

	/**
	 * Create a reference to a show flag variable
	 * @param CVarName must not be 0, e.g. "Show.PostProcessing"
	 * @param FlagName must not be 0, e.g. "PostProcessing"
	 * @param BitNumber in the memory defined by Force0MaskPtr and Force1MaskPtr
	 * @param Force0MaskPtr memory that contains the bits that should be forced to 0
	 * @param Force1MaskPtr memory that contains the bits that should be forced to 1
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	virtual IConsoleVariable* RegisterConsoleVariableBitRef(const TCHAR* CVarName, const TCHAR* FlagName, uint32 BitNumber, uint8* Force0MaskPtr, uint8* Force1MaskPtr, const TCHAR* Help, uint32 Flags = ECVF_Default) = 0;

	// ----------

	/**
	 * The sinks are only called if a change has been done since the last time
	 * Should be called in very few points:
	 *  - after ini file loading
	 *  - after user console input
	 *  - user initiated a console variable change (it needs to be clear to user that a cvar can change e.g. game options menu)
	 *  - beginning of Tick (to catch stray Set() calls, which are usually bad)
	 */
	virtual void CallAllConsoleVariableSinks() = 0;

	/**
	 * The registered command is executed at few defined points (see CallAllConsoleVariableSinks)
	 * @param Command
	 */
	virtual FConsoleVariableSinkHandle RegisterConsoleVariableSink_Handle(const FConsoleCommandDelegate& Command) = 0;

	/**
	 * The registered command is executed at few defined points (see CallAllConsoleVariableSinks)
	 * @param Command
	 */
	virtual void UnregisterConsoleVariableSink_Handle(FConsoleVariableSinkHandle Handle) = 0;

	// ----------

	/**
	 * Register a console command that takes no arguments
	 *
	 * @param	Name		The name of this command (must not be nullptr)
	 * @param	Help		Help text for this command
	 * @param	Command		The user function to call when this command is executed
	 * @param	Flags		Optional flags bitmask
	 */
	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandDelegate& Command, uint32 Flags = ECVF_Default) = 0;

	/**
	 * Register a console command that takes arguments
	 *
	 * @param	Name		The name of this command (must not be nullptr)
	 * @param	Help		Help text for this command
	 * @param	Command		The user function to call when this command is executed
	 * @param	Flags		Optional flags bitmask
	 */
	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsDelegate& Command, uint32 Flags = ECVF_Default) = 0;

	/**
	 * Register a console command that takes arguments
	 *
	 * @param	Name		The name of this command (must not be nullptr)
	 * @param	Help		Help text for this command
	 * @param	Command		The user function to call when this command is executed
	 * @param	Flags		Optional flags bitmask
	 */
	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldDelegate& Command, uint32 Flags = ECVF_Default) = 0;

	/**
	* Register a console command that takes arguments
	*
	* @param	Name		The name of this command (must not be nullptr)
	* @param	Help		Help text for this command
	* @param	Command		The user function to call when this command is executed
	* @param	Flags		Optional flags bitmask
	*/
	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldAndArgsDelegate& Command, uint32 Flags = ECVF_Default) = 0;
	
	/**
	* Register a console command that takes arguments
	*
	* @param	Name		The name of this command (must not be nullptr)
	* @param	Help		Help text for this command
	* @param	Command		The user function to call when this command is executed
	* @param	Flags		Optional flags bitmask
	*/
	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsAndOutputDeviceDelegate& Command, uint32 Flags = ECVF_Default) = 0;
	
	/**
	* Register a console command that takes arguments
	*
	* @param	Name		The name of this command (must not be nullptr)
	* @param	Help		Help text for this command
	* @param	Command		The user function to call when this command is executed
	* @param	Flags		Optional flags bitmask
	*/
	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldArgsAndOutputDeviceDelegate& Command, uint32 Flags = ECVF_Default) = 0;

	/**
	 * Register a console command that takes arguments
	 *
	 * @param	Name		The name of this command (must not be nullptr)
	 * @param	Help		Help text for this command
	 * @param	Command		The user function to call when this command is executed
	 * @param	Flags		Optional flags bitmask
	 */
	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithOutputDeviceDelegate& Command, uint32 Flags = ECVF_Default) = 0;

	/**
	 * Register a console command that is handles by an Exec functions (for auto completion)
	 *
	 * @param	Name		The name of this command (must not be nullptr)
	 * @param	Help		Help text for this command
	 * @param	Flags		Optional flags bitmask
	 */
	virtual IConsoleCommand* RegisterConsoleCommand(const TCHAR* Name, const TCHAR* Help, uint32 Flags = (uint32)ECVF_Default) = 0;

	/**
	 * Unregisters a console object, if that object was registered. O(n), n is the console object count
	 *
	 * @param ConsoleObject - object to remove
	 * @param bKeepState if the current state is kept in memory until a cvar with the same name is registered
	 */
	virtual void UnregisterConsoleObject( IConsoleObject* ConsoleObject, bool bKeepState = true) = 0;

	/**
	 * Unregisters a console variable or command by name, if an object of that name was registered.
	 *
	 * @param Name - name of object to remove
	 * @param bKeepState if the current state is kept in memory until a cvar with the same name is registered
	 */
	virtual void UnregisterConsoleObject(const TCHAR* Name, bool bKeepState = true) = 0;

	/**
	 * Find a console variable
	 * @param Name must not be 0
	 * @return 0 if the object wasn't found
	 */
	virtual IConsoleVariable* FindConsoleVariable(const TCHAR* Name, bool bTrackFrequentCalls = true) const = 0;

	/**
	* Find a console variable or command
	* @param Name must not be 0
	* @return 0 if the object wasn't found
	*/
	virtual IConsoleObject* FindConsoleObject(const TCHAR* Name, bool bTrackFrequentCalls = true) const = 0;

	/**
	 * Lookup the name of a console object by its pointer 
	 * @param Object to lookup. Must not be null.
	 * @return Name of the object, or an empty string if the object can't be found
	 */
	virtual FString FindConsoleObjectName(const IConsoleObject* Obj) const = 0;

	/**
	 * Find a typed console variable (faster access to the value, no virtual function call)
	 * @param Name must not be 0
	 * @return 0 if the object wasn't found
	 */
	TConsoleVariableData<bool>* FindTConsoleVariableDataBool(const TCHAR* Name) const
	{
		IConsoleVariable* P = FindConsoleVariable(Name);

		return P ? P->AsVariableBool() : 0;
	}

	/**
	 * Find a typed console variable (faster access to the value, no virtual function call)
	 * @param Name must not be 0
	 * @return 0 if the object wasn't found
	 */
	TConsoleVariableData<int32>* FindTConsoleVariableDataInt(const TCHAR* Name) const 
	{ 
		IConsoleVariable* P = FindConsoleVariable(Name); 
		
		return P ? P->AsVariableInt() : 0; 
	}

	/**
	 * Find a typed console variable (faster access to the value, no virtual function call)
	 * @param Name must not be 0
	 * @return 0 if the object wasn't found
	 */
	TConsoleVariableData<float>* FindTConsoleVariableDataFloat(const TCHAR* Name) const 
	{ 
		IConsoleVariable* P = FindConsoleVariable(Name); 

		return P ? P->AsVariableFloat() : 0; 
	}

	/**
	 * Find a typed console variable (faster access to the value, no virtual function call)
	 * @param Name must not be 0
	 * @return 0 if the object wasn't found
	 */
	TConsoleVariableData<FString>* FindTConsoleVariableDataString(const TCHAR* Name) const
	{
		IConsoleVariable* P = FindConsoleVariable(Name);

		return P ? P->AsVariableString() : 0;
	}


	/**
	 *  Iterate in O(n), not case sensitive, does not guarantee that UnregisterConsoleObject() will work in the loop
	 *  @param Visitor must not be 0
	 *  @param ThatStartsWith must not be 0 
	 */
	virtual void ForEachConsoleObjectThatStartsWith( const FConsoleObjectVisitor& Visitor, const TCHAR* ThatStartsWith = TEXT("")) const = 0;

	/**
	 *  Not case sensitive, does not guarantee that UnregisterConsoleObject() will work in the loop
	 *  @param Visitor must not be 0
	 *  @param ThatContains must not be 0 
	 */
	virtual void ForEachConsoleObjectThatContains(const FConsoleObjectVisitor& Visitor, const TCHAR* ThatContains) const = 0;

	/**
	 * Process user input
	 *  e.g.
	 *  "MyCVar" to get the current value of the console variable
	 *  "MyCVar -5.2" to set the value to -5.2
	 *  "MyCVar ?" to get the help text
	 *  @param	Input		must not be 0
	 *  @param	Ar			archive
	 *  @param	InWorld		world context
	 *  @return true if the command was recognized
	 */
	virtual bool ProcessUserConsoleInput(const TCHAR* Input, FOutputDevice& Ar, UWorld* InWorld) = 0;

	/**
	 * @param Input - must not be 0
	 */
	virtual void AddConsoleHistoryEntry(const TCHAR* Key, const TCHAR* Input) = 0;
	
	/**
	 */
	virtual void GetConsoleHistory(const TCHAR* Key, TArray<FString>& Out) = 0; 

	/**
	 * Check if a name (command or variable) has been registered with the console manager
	 * @param Name - Name to check. Must not be 0
	 */
	virtual bool IsNameRegistered(const TCHAR* Name) const = 0;

	// currently only for render thread
	// @param InCallback 0 to disable the callbacks
	virtual void RegisterThreadPropagation(uint32 ThreadId = 0, IConsoleThreadPropagation* InCallback = 0) = 0;

	/** Returns the singleton for the console manager **/
	inline static IConsoleManager& Get()
	{
		if (!Singleton)
		{
			SetupSingleton();
			check(Singleton != nullptr);
		}
		return *Singleton;
	}

#if ALLOW_OTHER_PLATFORM_CONFIG
	/**
	 * Walks over the best set of ini sections for another platform that can be used to emulate cvars as the platform
	 * will have set
	 * It also won't include any UserSettings
	 *
	 * @param PlatformName The platform name (the ini name, so Windows, not Win64)
	 * @param DeviceProfileName If this is non-empty, the given deviceprofile will be loaded from the platform's inis and inserted into the CVars
	 * @param Visit the callback to run for each CVar found
	 */
	static CORE_API bool VisitPlatformCVarsForEmulation(FName PlatformName, const FString& DeviceProfileName, TFunctionRef<void(const FString& CVarName, const FString& CVarValue, EConsoleVariableFlags SetBy)> Visit);
	
	/**
	 * Loads all platform cvars, which can be retrieved with IConsoleVariable::GetPlatformValueVariable. If DeviceProfileName is empty, it will use the
	 *  DevicePlatform named the same as the Platform
	 */
	virtual void LoadAllPlatformCVars(FName PlatformName, const FString& DeviceProfileName=FString()) = 0;
	
	/**
	 * Applies the cvars from the platform/DeviceProfile pair (can be blank to use the platform's named DP) and sets into the SetByPreview  priority, with the PreviewModeTag
	 * to be unset later
	 */
	virtual void PreviewPlatformCVars(FName PlatformName, const FString& DeviceProfileName, FName PreviewModeTag) = 0;

	/**
	 * Applies the cvars from the platform/DeviceProfile pair (can be blank to use the platform's named DP) and sets at the given SetBy priority, with the PreviewModeTag
	 * to be unset later. RequiredFlags must be present ont he CVar to be set, unless DisallowedFlags is present, then it will not set it
	 */
	virtual void StompPlatformCVars(FName PlatformName, const FString& DeviceProfileName, FName Tag, EConsoleVariableFlags SetBy, EConsoleVariableFlags RequiredFlags, EConsoleVariableFlags DisallowedFlags) = 0;

	/**
	 * Empties the cache for the given Platform/DP for all CVars. If using NAME_None for Platofrm, this will wipe every cached platform/DP value. If DeviceProfileName
	 * is empty, it will use the PlatfomName as the DeviceProfile name
	 */
	virtual void ClearAllPlatformCVars(FName PlatformName=NAME_None, const FString& DeviceProfileName=FString()) = 0;

#endif

	/**
	  * When a plugin is unmounted, it needs to unset cvars that it had set when it was mounted. This will unset and fixup all
	  * variables with the given Tag
	  * @param Priority If set, then the internal search for CVars is restricted to this SetBy. This is an optimization, only to be used if you _know_ that all CVars set with this tag were set at this priority. Any other priorites cannot be unset
	 */
	virtual void UnsetAllConsoleVariablesWithTag(FName Tag, EConsoleVariableFlags Priority=ECVF_SetByMask) = 0;

	/**
	* Add, replace, or remove cvars to match the passed in cvars
	*/
	virtual void BatchUpdateTag(FName Tag, const TMap<FName, FString>& CVarsAndValues) = 0;

	virtual FConsoleVariableMulticastDelegate& OnCVarUnregistered() = 0;
	virtual FConsoleObjectWithNameMulticastDelegate& OnConsoleObjectUnregistered() = 0;

protected:
	virtual ~IConsoleManager() = default;

private:
	/** Singleton for the console manager **/
	static CORE_API IConsoleManager* Singleton;

	/** Function to create the singleton **/
	static CORE_API void SetupSingleton();
};


/**
 * Utility class to modify a console variable's value for a certain scope and restore it to its previous value afterwards.
 * Type parameter may be any supported by IConsoleVariable::GetValue, i.e. bool, int32, float, FString.
 */
template <typename T>
struct TGuardConsoleVariable
{
	T PreviousValue;
	IConsoleVariable* ConsoleVariable;
	IConsoleVariable::FSetContext SetContext;
	
	TGuardConsoleVariable(const TCHAR* ConsoleVariableName, const T& NewValue, EConsoleVariableFlags Flags = ECVF_SetByCode, FName Tag = NAME_None)
		: TGuardConsoleVariable(IConsoleManager::Get().FindConsoleVariable(ConsoleVariableName), NewValue, Flags, Tag)
	{
		ensureMsgf(ConsoleVariable, TEXT("TGuardConsoleVariable: Trying to set console variable \"%s\" but it doesn't exist"), ConsoleVariableName);
	}

	TGuardConsoleVariable(IConsoleVariable* ConsoleVariable, const T& NewValue, EConsoleVariableFlags Flags = ECVF_SetByCode, FName Tag = NAME_None)
		: ConsoleVariable(ConsoleVariable)
		, SetContext(Flags, Tag)
	{
		if (ConsoleVariable)
		{
			ConsoleVariable->GetValue(PreviousValue);
			ConsoleVariable->Set(NewValue, SetContext);
		}
	}

	TGuardConsoleVariable(const TGuardConsoleVariable&) = delete;

	TGuardConsoleVariable(TGuardConsoleVariable&& Right)
		: PreviousValue(::MoveTemp(Right.PreviousValue))
		, ConsoleVariable(Right.ConsoleVariable)
	{
		Right.ConsoleVariable = nullptr;
	}

	TGuardConsoleVariable& operator=(const TGuardConsoleVariable&) = delete;

	TGuardConsoleVariable& operator=(TGuardConsoleVariable&&) = delete;

	~TGuardConsoleVariable()
	{
		if (ConsoleVariable)
		{
			ConsoleVariable->Set(PreviousValue, SetContext);
		}
	}
};

/**
 * auto registering console variable sinks (register a callback function that is called when ever a cvar is changes by the user, changes are grouped and happen in specific engine spots during the frame/main loop)
 */
class FAutoConsoleVariableSink
{
public:
	/** Constructor, saves the argument for future removal from the console variable system **/
	FAutoConsoleVariableSink(FConsoleCommandDelegate&& InCommand)
		: Command(MoveTemp(InCommand))
	{
		Handle = IConsoleManager::Get().RegisterConsoleVariableSink_Handle(Command);
	}

	/** Constructor, saves the argument for future removal from the console variable system **/
	FAutoConsoleVariableSink(const FConsoleCommandDelegate& InCommand)
		: Command(InCommand)
	{
		Handle = IConsoleManager::Get().RegisterConsoleVariableSink_Handle(Command);
	}
	/** Destructor, removes the console variable sink **/
	virtual ~FAutoConsoleVariableSink() = default;

	FConsoleCommandDelegate Command;
	FConsoleVariableSinkHandle Handle;
};


/**
 * Base class for autoregistering console commands.
 */
class FAutoConsoleObject
{
protected:
	/** Constructor, saves the argument for future removal from the console variable system **/
	FAutoConsoleObject(IConsoleObject* InTarget)
		: Target(InTarget)
	{
		check(Target);
		if (Target->TestFlags(ECVF_GeneralShaderChange))
		{
			AccessGeneralShaderChangeCvars().Add(this);
		}
		else if (Target->TestFlags(ECVF_MobileShaderChange))
		{
			AccessMobileShaderChangeCvars().Add(this);
		}
		else if (Target->TestFlags(ECVF_DesktopShaderChange))
		{
			AccessDesktopShaderChangeCvars().Add(this);
		}
	}
public:
	/** Destructor, removes the console object **/
	virtual ~FAutoConsoleObject()
	{
#if UE_MERGED_MODULES
		const bool bKeepCVarObject = false; // do not keep the console object in memory, as it may cause crashes if we reload the object later
#else
		const bool bKeepCVarObject = true; // do not delete console objects by default as some existing code assumes they're kept
#endif
		IConsoleManager::Get().UnregisterConsoleObject(Target, bKeepCVarObject);
	}

	static CORE_API TArray<const FAutoConsoleObject*>& AccessGeneralShaderChangeCvars();
	static CORE_API TArray<const FAutoConsoleObject*>& AccessMobileShaderChangeCvars();
	static CORE_API TArray<const FAutoConsoleObject*>& AccessDesktopShaderChangeCvars();

	/** returns the contained console object as an IConsoleVariable **/
	inline IConsoleVariable* AsVariable()
	{
		checkSlow(Target->AsVariable());
		return static_cast<IConsoleVariable*>(Target);
	}
	/** returns the contained console object as an IConsoleVariable **/
	inline const IConsoleVariable* AsVariable() const
	{
		checkSlow(Target->AsVariable());
		return static_cast<const IConsoleVariable*>(Target);
	}

private:
	/** Contained console object, cannot be 0 **/
	IConsoleObject* Target;
};

#if !NO_CVARS
/**
 * Autoregistering float, int or string console variable
 */
class FAutoConsoleVariable : private FAutoConsoleObject
{
	CORE_API FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, bool DefaultValue, const TCHAR* Help);
	CORE_API FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, int32 DefaultValue, const TCHAR* Help);
	CORE_API FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, float DefaultValue, const TCHAR* Help);
	CORE_API FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, const TCHAR* DefaultValue, const TCHAR* Help);
	CORE_API FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, bool DefaultValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback);
	CORE_API FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, int32 DefaultValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback);
	CORE_API FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, float DefaultValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback);
	CORE_API FAutoConsoleVariable(uint32 Flags, const TCHAR* Name, const TCHAR* DefaultValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback);
public:
	/**
	 * Create a bool console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariable(const TCHAR* Name, bool DefaultValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariable(Flags, Name, DefaultValue, UE_CVAR_HELP_TEXT_PARAM(Help)) {}
	/**
	 * Create a int console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariable(const TCHAR* Name, int32 DefaultValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariable(Flags, Name, DefaultValue, UE_CVAR_HELP_TEXT_PARAM(Help)) {}
	/**
	 * Create a float console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariable(const TCHAR* Name, float DefaultValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariable(Flags, Name, DefaultValue, UE_CVAR_HELP_TEXT_PARAM(Help)) {}
	/**
	 * Create a string console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariable(const TCHAR* Name, const TCHAR* DefaultValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariable(Flags, Name, DefaultValue, UE_CVAR_HELP_TEXT_PARAM(Help)) {}
	/**
	 * Create a bool console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Callback Delegate called when the variable changes. @see IConsoleVariable::SetOnChangedCallback
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariable(const TCHAR* Name, bool DefaultValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariable(Flags, Name, DefaultValue, UE_CVAR_HELP_TEXT_PARAM(Help), Callback) {}
	/**
	 * Create a int console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Callback Delegate called when the variable changes. @see IConsoleVariable::SetOnChangedCallback
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariable(const TCHAR* Name, int32 DefaultValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariable(Flags, Name, DefaultValue, UE_CVAR_HELP_TEXT_PARAM(Help), Callback) {}
	/**
	 * Create a float console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Callback Delegate called when the variable changes. @see IConsoleVariable::SetOnChangedCallback
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariable(const TCHAR* Name, float DefaultValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariable(Flags, Name, DefaultValue, UE_CVAR_HELP_TEXT_PARAM(Help), Callback) {}
	/**
	 * Create a string console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Callback Delegate called when the variable changes. @see IConsoleVariable::SetOnChangedCallback
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariable(const TCHAR* Name, const TCHAR* DefaultValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariable(Flags, Name, DefaultValue, UE_CVAR_HELP_TEXT_PARAM(Help), Callback) {}

	virtual ~FAutoConsoleVariable() {}

	/** Dereference back to a console variable**/
	UE_FORCEINLINE_HINT IConsoleVariable& operator*()
	{
		return *AsVariable();
	}
	UE_FORCEINLINE_HINT const IConsoleVariable& operator*() const
	{
		return *AsVariable();
	}
	/** Dereference back to a console variable**/
	UE_FORCEINLINE_HINT IConsoleVariable* operator->()
	{
		return AsVariable();
	}
	UE_FORCEINLINE_HINT const IConsoleVariable* operator->() const
	{
		return AsVariable();
	}
};
#else
class FAutoConsoleVariable
{
public:
	FAutoConsoleVariable(const TCHAR* Name, int32 DefaultValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
	{
	}

	FAutoConsoleVariable(const TCHAR* Name, float DefaultValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
	{
	}

	FAutoConsoleVariable(const TCHAR* Name, const TCHAR* DefaultValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
	{
	}
};
#endif


enum class EShadowCVarBehavior : uint8
{
	// skip all messaging
	NoMessaging,
	// log a warning on first usage
	Warn,
	// log an error on first usage
	Error,
	// throw an ensure on first usage
	Ensure,
	// throw an assertion on first usage
	Assert,
};

/**
 * Disable deprecation warnings for access to cvars for the duration of the current scope, enabling internal use of deprecated cvars without generating warnings / ensures.
 * Analogous to the use case for the pragmas for deprecated code.
 */
struct FCVarDisableDeprecationWarningScope
{
	/**
	 * if bInCondition is false, the scope does nothing such that it can be conditionally used.
	 */
	CORE_API FCVarDisableDeprecationWarningScope(bool bInCondition = true);
	CORE_API ~FCVarDisableDeprecationWarningScope();

	CORE_API static bool IsDisabled();

private:
	const bool bCondition = true;
};

struct FCVarDeprecationArgs
{
	// The first UE version when this was deprecated
	const TCHAR* Version = TEXT("");
	// How to handle uses of the deprecated cvar  - defaults to ensure
	EShadowCVarBehavior UsageBehavior = EShadowCVarBehavior::Ensure;
	 // Optional deprecation message - defaults to empty / auto-generated.
	const TCHAR* Message = TEXT("");
};


#if !NO_CVARS
/**
 * Autoregistering float, int, bool, FString REF variable class...this changes that value when the console variable is changed. 
 */
class FAutoConsoleVariableRef : private FAutoConsoleObject
{
	CORE_API FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, int32& RefValue, const TCHAR* Help);
	CORE_API FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, float& RefValue, const TCHAR* Help);
	CORE_API FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, bool& RefValue, const TCHAR* Help);
	CORE_API FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, FString& RefValue, const TCHAR* Help);
	CORE_API FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, FName& RefValue, const TCHAR* Help);
	CORE_API FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, int32& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback);
	CORE_API FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, float& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback);
	CORE_API FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, bool& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback);
	CORE_API FAutoConsoleVariableRef(uint32 Flags, const TCHAR* Name, FString& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback);

	CORE_API FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, int32& RefValue, uint32 Flags);
	CORE_API FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, float& RefValue, uint32 Flags);
	CORE_API FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, bool& RefValue, uint32 Flags);
	CORE_API FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, FString& RefValue, uint32 Flags);
	CORE_API FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, FName& RefValue, uint32 Flags);
	CORE_API FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, int32& RefValue, const FConsoleVariableDelegate& Callback, uint32 Flags);
	CORE_API FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, float& RefValue, const FConsoleVariableDelegate& Callback, uint32 Flags);
	CORE_API FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, bool& RefValue, const FConsoleVariableDelegate& Callback, uint32 Flags);
	CORE_API FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, FString& RefValue, const FConsoleVariableDelegate& Callback, uint32 Flags);

public:
	/**
	 * Create a reference to a int console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariableRef(const TCHAR* Name, int32& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(Flags, Name, RefValue, UE_CVAR_HELP_TEXT_PARAM(Help)) {}
	FORCEINLINE FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, int32& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(DeprecationArgs, Name, RefValue, Flags) {}

	/**
	 * Create a reference to a float console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariableRef(const TCHAR* Name, float& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(Flags, Name, RefValue, UE_CVAR_HELP_TEXT_PARAM(Help)) {}
	FORCEINLINE FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, float& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(DeprecationArgs, Name, RefValue, Flags) {}

	/**
	 * Create a reference to a bool console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariableRef(const TCHAR* Name, bool& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(Flags, Name, RefValue, UE_CVAR_HELP_TEXT_PARAM(Help)) {}
	FORCEINLINE FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, bool& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(DeprecationArgs, Name, RefValue, Flags) {}
	/**
	 * Create a reference to a FString console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariableRef(const TCHAR* Name, FString& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(Flags, Name, RefValue, UE_CVAR_HELP_TEXT_PARAM(Help)) {}
	FORCEINLINE FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, FString& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(DeprecationArgs, Name, RefValue, Flags) {}

	/**
	 * Create a reference to a FName console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariableRef(const TCHAR* Name, FName& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(Flags, Name, RefValue, UE_CVAR_HELP_TEXT_PARAM(Help)) {}
	FORCEINLINE FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, FName& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(DeprecationArgs, Name, RefValue, Flags) {}

	/**
	 * Create a reference to a int console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Callback Delegate called when the variable changes. @see IConsoleVariable::SetOnChangedCallback
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariableRef(const TCHAR* Name, int32& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(Flags, Name, RefValue, UE_CVAR_HELP_TEXT_PARAM(Help), Callback) {}
	FORCEINLINE FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, int32& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(DeprecationArgs, Name, RefValue, Callback, Flags) {}

	/**
	 * Create a reference to a float console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Callback Delegate called when the variable changes. @see IConsoleVariable::SetOnChangedCallback
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariableRef(const TCHAR* Name, float& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(Flags, Name, RefValue, UE_CVAR_HELP_TEXT_PARAM(Help), Callback) {}
	FORCEINLINE FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, float& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(DeprecationArgs, Name, RefValue, Callback, Flags) {}

	/**
	 * Create a reference to a bool console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Callback Delegate called when the variable changes. @see IConsoleVariable::SetOnChangedCallback
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariableRef(const TCHAR* Name, bool& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(Flags, Name, RefValue, UE_CVAR_HELP_TEXT_PARAM(Help), Callback) {}
	FORCEINLINE FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, bool& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(DeprecationArgs, Name, RefValue, Callback, Flags) {}

	/**
	 * Create a reference to a FString console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Callback Delegate called when the variable changes. @see IConsoleVariable::SetOnChangedCallback
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE FAutoConsoleVariableRef(const TCHAR* Name, FString& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(Flags, Name, RefValue, UE_CVAR_HELP_TEXT_PARAM(Help), Callback) {}
	FORCEINLINE FAutoConsoleVariableRef(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, FString& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
		: FAutoConsoleVariableRef(DeprecationArgs, Name, RefValue, Callback, Flags) {}

	virtual ~FAutoConsoleVariableRef()
	{
		AsVariable()->ClearOnChangedCallback();
	}

	/** Dereference back to a variable**/
	UE_FORCEINLINE_HINT IConsoleVariable& operator*()
	{
		return *AsVariable();
	}
	UE_FORCEINLINE_HINT const IConsoleVariable& operator*() const
	{
		return *AsVariable();
	}
	/** Dereference back to a variable**/
	UE_FORCEINLINE_HINT IConsoleVariable* operator->()
	{
		return AsVariable();
	}
	UE_FORCEINLINE_HINT const IConsoleVariable* operator->() const
	{
		return AsVariable();
	}
};
#else
class FAutoConsoleVariableRef
{
public:
	FAutoConsoleVariableRef(const TCHAR* Name, int32& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
	{
	}

	FAutoConsoleVariableRef(const TCHAR* Name, float& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
	{
	}

	FAutoConsoleVariableRef(const TCHAR* Name, bool& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
	{
	}

	FAutoConsoleVariableRef(const TCHAR* Name, FString& RefValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
	{
	}

	FAutoConsoleVariableRef(const TCHAR* Name, int32& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
	{
	}

	FAutoConsoleVariableRef(const TCHAR* Name, float& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
	{
	}
	
	FAutoConsoleVariableRef(const TCHAR* Name, bool& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
	{
	}

	FAutoConsoleVariableRef(const TCHAR* Name, FString& RefValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
	{
	}
};
#endif // NO_CVARS


// currently only supports main and render thread
// optimized for read access speed (no virtual function call and no thread handling if using the right functions)
// T: int32, float
template <class T>
class TConsoleVariableData
{
public:
	// constructor
	TConsoleVariableData(const T DefaultValue)
	{
		for(uint32 i = 0; i < UE_ARRAY_COUNT(ShadowedValue); ++i)
		{
			ShadowedValue[i] = DefaultValue;
		}
	}

	// faster than GetValueOnAnyThread()
	T GetValueOnGameThread() const
	{
		UE::AccessDetection::ReportAccess(UE::AccessDetection::EType::CVar);
		// compiled out in shipping for performance (we can change in development later), if this get triggered you need to call GetValueOnRenderThread() or GetValueOnAnyThread(), the last one is a bit slower
		cvarCheckCode(ensure(GetShadowIndex() == 0));	// ensure to not block content creators, #if to optimize in shipping
		return ShadowedValue[0];
	}

	// faster than GetValueOnAnyThread()
	T GetValueOnRenderThread() const
	{
		UE::AccessDetection::ReportAccess(UE::AccessDetection::EType::CVar);
#if !defined(__clang__) // @todo Mac: figure out how to make this compile
		// compiled out in shipping for performance (we can change in development later), if this get triggered you need to call GetValueOnGameThread() or GetValueOnAnyThread(), the last one is a bit slower
		cvarCheckCode(ensure(IsInParallelRenderingThread()));	// ensure to not block content creators, #if to optimize in shipping
#endif
		return ShadowedValue[1];
	}

	// convenient, for better performance consider using GetValueOnGameThread() or GetValueOnRenderThread()
	T GetValueOnAnyThread(bool bForceGameThread = false) const
	{
		UE::AccessDetection::ReportAccess(UE::AccessDetection::EType::CVar);
		return ShadowedValue[GetShadowIndex(bForceGameThread)];
	}

private: // ----------------------------------------------------

	// [0]:main thread, [1]: render thread, having them both in the same cache line should only hurt on write which happens rarely for cvars
	T ShadowedValue[2];

	// @return 0:main thread, 1: render thread, later more
	static uint32 GetShadowIndex(bool bForceGameThread = false)
	{	
		if (bForceGameThread)
		{
			cvarCheckCode(ensure(!IsInActualRenderingThread()));
			return 0;
		}
		return IsInParallelGameThread() || IsInGameThread() ? 0 : 1;
	}

	// needed for FConsoleVariable and FConsoleVariableRef2, intentionally not public
	T& GetReferenceOnAnyThread(bool bForceGameThread = false)
	{
		UE::AccessDetection::ReportAccess(UE::AccessDetection::EType::CVar);
		return ShadowedValue[GetShadowIndex(bForceGameThread)];
	}

	template<class T2> friend class FConsoleVariable;
	template<class T2> friend class TAutoConsoleVariable;
};

#if !NO_CVARS
/**
 * Autoregistering float, int variable class...this changes that value when the console variable is changed. 
 */
template <class T>
class TAutoConsoleVariable : public FAutoConsoleObject
{
	FORCENOINLINE TAutoConsoleVariable(uint32 Flags, const TCHAR* Name, const T& DefaultValue, const TCHAR* Help)
		: TAutoConsoleVariable(IConsoleManager::Get().RegisterConsoleVariable(Name, DefaultValue, Help, Flags))
	{
	}

	CORE_API TAutoConsoleVariable(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, const T& DefaultValue, uint32 Flags);

	FORCENOINLINE TAutoConsoleVariable(uint32 Flags, const TCHAR* Name, const T& DefaultValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback)
		: TAutoConsoleVariable(Flags, Name, DefaultValue, Help)
	{
		AsVariable()->SetOnChangedCallback(Callback);
	}

	FORCENOINLINE TAutoConsoleVariable(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, const T& DefaultValue, const FConsoleVariableDelegate& Callback, uint32 Flags)
		: TAutoConsoleVariable(DeprecationArgs, Name, DefaultValue, Flags)
	{
		AsVariable()->SetOnChangedCallback(Callback);
	}

	FORCENOINLINE TAutoConsoleVariable(uint32 Flags, const TCHAR* Name, const T& DefaultValue, TFunction<T()> Getter, TFunction<void(const T&)> Setter, const TCHAR* Help)
		: TAutoConsoleVariable(IConsoleManager::Get().RegisterDelegatedConsoleVariable(Name, DefaultValue, MoveTemp(Getter), MoveTemp(Setter), Help, Flags))
	{
	}

public:

	TAutoConsoleVariable(IConsoleVariable* ExistingObject)
		: FAutoConsoleObject(ExistingObject)
	{
		if constexpr (std::is_same_v<T, bool>)
		{
			Ref = AsVariable()->AsVariableBool();
		}
		else if constexpr (std::is_same_v<T, int32>)
		{
			Ref = AsVariable()->AsVariableInt();
		}
		else if constexpr (std::is_same_v<T, float>)
		{
			Ref = AsVariable()->AsVariableFloat();
		}
		else if constexpr (std::is_same_v<T, FString>)
		{
			Ref = AsVariable()->AsVariableString();
		}
		else
		{
			static_assert(sizeof(T) == 0, "Not supported");
		}
	}

	/**
	 * Create a float, int or string console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE TAutoConsoleVariable(const TCHAR* Name, const T& DefaultValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: TAutoConsoleVariable(Flags, Name, DefaultValue, UE_CVAR_HELP_TEXT_PARAM(Help))
	{
	}

	FORCEINLINE TAutoConsoleVariable(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, const T& DefaultValue, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: TAutoConsoleVariable(DeprecationArgs, Name, DefaultValue, Flags)
	{
	}
	
	/**
	 * Create a float, int or string console variable
	 * @param Name must not be 0
	 * @param Help must not be 0
	 * @param Callback Delegate called when the variable changes. @see IConsoleVariable::SetOnChangedCallback
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE TAutoConsoleVariable(const TCHAR* Name, const T& DefaultValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
		: TAutoConsoleVariable(Flags, Name, DefaultValue, UE_CVAR_HELP_TEXT_PARAM(Help), Callback)
	{
	}

	FORCEINLINE TAutoConsoleVariable(const FCVarDeprecationArgs& DeprecationArgs, const TCHAR* Name, const T& DefaultValue, const TCHAR* Help, const FConsoleVariableDelegate& Callback, uint32 Flags = ECVF_Default)
		: TAutoConsoleVariable(DeprecationArgs, Name, DefaultValue, Callback, Flags)
	{
	}

	/**
	 * Create a console variable that delegates get/set to external functions.
	 * The Getter is called on every read; the Setter is called on every write.
	 * ECVF_RenderThreadSafe is NOT supported with delegate variables.
	 *
	 * @param Name must not be 0
	 * @param DefaultValue initial value; the Getter is authoritative after construction
	 * @param Getter called to read the current value
	 * @param Setter called to write a new value
	 * @param Help must not be 0
	 * @param Flags bitmask combined from EConsoleVariableFlags
	 */
	FORCEINLINE TAutoConsoleVariable(const TCHAR* Name, const T& DefaultValue, TFunction<T()> Getter, TFunction<void(const T&)> Setter, const TCHAR* Help, uint32 Flags = ECVF_Default)
		: TAutoConsoleVariable(Flags, Name, DefaultValue, MoveTemp(Getter), MoveTemp(Setter), UE_CVAR_HELP_TEXT_PARAM(Help))
	{
	}

	virtual ~TAutoConsoleVariable()
	{
		AsVariable()->ClearOnChangedCallback();
	}

	T GetValueOnGameThread() const
	{
		if (Ref)
		{
			return Ref->GetValueOnGameThread();
		}
		T Result;
		AsVariable()->GetValue(Result);
		return Result;
	}

	T GetValueOnRenderThread() const
	{
		if (Ref)
		{
			return Ref->GetValueOnRenderThread();
		}
		T Result;
		AsVariable()->GetValue(Result);
		return Result;
	}

	T GetValueOnAnyThread(bool bForceGameThread = false) const
	{
		if (Ref)
		{
			return Ref->GetValueOnAnyThread(bForceGameThread);
		}
		T Result;
		AsVariable()->GetValue(Result);
		return Result;
	}
	
	/** Dereference back to a variable**/
	UE_FORCEINLINE_HINT IConsoleVariable& operator*()
	{
		return *AsVariable();
	}
	UE_FORCEINLINE_HINT const IConsoleVariable& operator*() const
	{
		return *AsVariable();
	}
	/** Dereference back to a variable**/
	UE_FORCEINLINE_HINT IConsoleVariable* operator->()
	{
		return AsVariable();
	}
	UE_FORCEINLINE_HINT const IConsoleVariable* operator->() const
	{
		return AsVariable();
	}
private:
	TConsoleVariableData<T>* Ref = nullptr;
};

#else
template <class T>
class TAutoConsoleVariable : public IConsoleVariable
{
public:
	TAutoConsoleVariable(const TCHAR* Name, const T& DefaultValue, const TCHAR* InHelp, uint32 InFlags = ECVF_Default)
		: Value(DefaultValue), Flags((EConsoleVariableFlags)InFlags)
	{
	}

	T GetValueOnGameThread() const
	{
		return Value.GetValueOnGameThread();
	}

	T GetValueOnRenderThread() const
	{
		return Value.GetValueOnRenderThread();
	}

	T GetValueOnAnyThread(bool bForceGameThread = false) const
	{
		return Value.GetValueOnAnyThread(bForceGameThread);
	}
	
	IConsoleVariable& operator*()
	{
		return *AsVariable();
	}

	const IConsoleVariable& operator*() const
	{
		return *AsVariable();
	}

	IConsoleVariable* operator->()
	{
		return AsVariable();
	}

	const IConsoleVariable* operator->() const
	{
		return AsVariable();
	}

	IConsoleVariable*		AsVariable()		{ return this; }
	const IConsoleVariable* AsVariable() const	{ return this; }

	virtual class TConsoleVariableData<int32>*		AsVariableInt()		override { return AsImpl<int32>(); }
	virtual class TConsoleVariableData<float>*		AsVariableFloat()	override { return AsImpl<float>(); }
	virtual class TConsoleVariableData<FString>*	AsVariableString()	override { return AsImpl<FString>(); }

	virtual bool		IsVariableInt() const override	{ return std::is_same_v<int32, T>; }
	virtual int32		GetInt()		const override	{ return GetImpl<int32>(); }
	virtual float		GetFloat()		const override	{ return GetImpl<float>(); }
	virtual FString		GetString()		const override	{ return GetImpl<FString>(); }
	virtual bool		GetBool()		const override { return GetImpl<bool>(); }

#if UE_WITH_CVAR_HELP_TEXT
	virtual const TCHAR* GetHelp() const override
	{
		return TEXT("NO_CVARS, no help");
	}

	virtual void SetHelp(const TCHAR* InHelp) override
	{
		check(false);
	}

	virtual FText GetDetailedHelp() const override
	{
		return INVTEXT("NO_CVARS, no help");
	}
#endif

	virtual void Release() override
	{
		check(false);
	}

	virtual void SetOnChangedCallback(const FConsoleVariableDelegate &) override
	{
		check(false);
	}

	virtual FConsoleVariableMulticastDelegate& OnChangedDelegate()override
	{
		static FConsoleVariableMulticastDelegate Dummy;
		check(false);
		return Dummy;
	}

	virtual EConsoleVariableFlags GetFlags() const override
	{
		return Flags;
	}

	virtual void SetFlags(const EConsoleVariableFlags InFlags) override
	{
		Flags = InFlags;
	}

	virtual void Set(const TCHAR* InValue, EConsoleVariableFlags SetBy) override
	{
		LexFromString(Value.ShadowedValue[0], InValue);
	}

private:
	TConsoleVariableData<T> Value;
	FString Help;
	EConsoleVariableFlags Flags = EConsoleVariableFlags::ECVF_Default;

	template<class Y>
	Y GetImpl() const
	{
		if constexpr (std::is_same_v<T, Y>)
		{
			return GetValueOnAnyThread();
		}
		else
		{
			check(false);
			return Y();
		}
	}

	template<class Y>
	TConsoleVariableData<T>* AsImpl()
	{
		if constexpr (std::is_same_v<T, Y>)
		{
			return &Value;
		}
		else
		{
			check(false);
			return nullptr;
		}
	}
};
#endif // NO_CVARS

#if !NO_CVARS
/**
 * Autoregistering console command
 */
class FAutoConsoleCommand : private FAutoConsoleObject
{
	CORE_API FAutoConsoleCommand(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandDelegate& Command);
	CORE_API FAutoConsoleCommand(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsDelegate& Command);
	CORE_API FAutoConsoleCommand(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithOutputDeviceDelegate& Command);
	CORE_API FAutoConsoleCommand(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsAndOutputDeviceDelegate& Command);
	CORE_API FAutoConsoleCommand(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldDelegate& Command);
	CORE_API FAutoConsoleCommand(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldArgsAndOutputDeviceDelegate& Command);
public:
	/**
	 * Register a console command that takes no arguments
	 *
	 * @param	Name		The name of this command (must not be nullptr)
	 * @param	Help		Help text for this command
	 * @param	Command		The user function to call when this command is executed
	 * @param	Flags		Optional flags bitmask
	 */
	FORCEINLINE FAutoConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandDelegate& Command, uint32 Flags = ECVF_Default)
		: FAutoConsoleCommand(Flags, Name, UE_CVAR_HELP_TEXT_PARAM(Help), Command) {}

	/**
	 * Register a console command that takes arguments
	 *
	 * @param	Name		The name of this command (must not be nullptr)
	 * @param	Help		Help text for this command
	 * @param	Command		The user function to call when this command is executed
	 * @param	Flags		Optional flags bitmask
	 */
	FORCEINLINE FAutoConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsDelegate& Command, uint32 Flags = ECVF_Default)
		: FAutoConsoleCommand(Flags, Name, UE_CVAR_HELP_TEXT_PARAM(Help), Command) {}

	/**
	* Register a console command that takes an output device
	*
	* @param	Name		The name of this command (must not be nullptr)
	* @param	Help		Help text for this command
	* @param	Command		The user function to call when this command is executed
	* @param	Flags		Optional flags bitmask
	*/
	FORCEINLINE FAutoConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithOutputDeviceDelegate& Command, uint32 Flags = ECVF_Default)
		: FAutoConsoleCommand(Flags, Name, UE_CVAR_HELP_TEXT_PARAM(Help), Command) {}

    /**
    * Register a console command that takes an output device
    *
    * @param	Name		The name of this command (must not be nullptr)
    * @param	Help		Help text for this command
    * @param	Command		The user function to call when this command is executed
    * @param	Flags		Optional flags bitmask
    */
	FORCEINLINE FAutoConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsAndOutputDeviceDelegate& Command, uint32 Flags = ECVF_Default)
		: FAutoConsoleCommand(Flags, Name, UE_CVAR_HELP_TEXT_PARAM(Help), Command) {}

	/**
	* Register a console command that takes a world argument
	*
	* @param	Name		The name of this command (must not be nullptr)
	* @param	Help		Help text for this command
	* @param	Command		The user function to call when this command is executed
	* @param	Flags		Optional flags bitmask
	*/
	FORCEINLINE FAutoConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldDelegate& Command, uint32 Flags = ECVF_Default)
		: FAutoConsoleCommand(Flags, Name, UE_CVAR_HELP_TEXT_PARAM(Help), Command) {}

	/**
	* Register a console command that takes arguments, a world argument and an output device
	*
	* @param	Name		The name of this command (must not be nullptr)
	* @param	Help		Help text for this command
	* @param	Command		The user function to call when this command is executed
	* @param	Flags		Optional flags bitmask
	*/
	FORCEINLINE FAutoConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldArgsAndOutputDeviceDelegate& Command, uint32 Flags = ECVF_Default)
		: FAutoConsoleCommand(Flags, Name, UE_CVAR_HELP_TEXT_PARAM(Help), Command) {}

	virtual ~FAutoConsoleCommand() {}
};
#else
class FAutoConsoleCommand
{
public:
	FAutoConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandDelegate& Command, uint32 Flags = ECVF_Default)
	{
	}

	FAutoConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsDelegate& Command, uint32 Flags = ECVF_Default)
	{
	}

	FAutoConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithOutputDeviceDelegate& Command, uint32 Flags = ECVF_Default)
	{
	}

	FAutoConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsAndOutputDeviceDelegate& Command, uint32 Flags = ECVF_Default)
	{
	}

	FAutoConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldDelegate& Command, uint32 Flags = ECVF_Default)
	{
	}

	FAutoConsoleCommand(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldArgsAndOutputDeviceDelegate& Command, uint32 Flags = ECVF_Default)
	{
	}
};
#endif


#if !NO_CVARS
/**
 * Autoregistering console command with a world
 */
class FAutoConsoleCommandWithWorld : private FAutoConsoleObject
{
	CORE_API FAutoConsoleCommandWithWorld(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldDelegate& Command);
public:
	/**
	 * Register a console command that takes a world argument
	 *
	 * @param	Name		The name of this command (must not be nullptr)
	 * @param	Help		Help text for this command
	 * @param	Command		The user function to call when this command is executed
	 * @param	Flags		Optional flags bitmask
	 */
	FORCEINLINE FAutoConsoleCommandWithWorld(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldDelegate& Command, uint32 Flags = ECVF_Default)
		: FAutoConsoleCommandWithWorld(Flags, Name, UE_CVAR_HELP_TEXT_PARAM(Help), Command) {}

	virtual ~FAutoConsoleCommandWithWorld() {}
};

/**
 * Autoregistering console command with a world and arguments
 */
class FAutoConsoleCommandWithWorldAndArgs : private FAutoConsoleObject
{
	CORE_API FAutoConsoleCommandWithWorldAndArgs(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldAndArgsDelegate& Command);
public:	
	/**
	 * Register a console command that takes arguments and a world argument
	 *
	 * @param	Name		The name of this command (must not be nullptr)
	 * @param	Help		Help text for this command
	 * @param	Command		The user function to call when this command is executed
	 * @param	Flags		Optional flags bitmask
	 */
	FORCEINLINE FAutoConsoleCommandWithWorldAndArgs(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldAndArgsDelegate& Command, uint32 Flags = ECVF_Default)
		: FAutoConsoleCommandWithWorldAndArgs(Flags, Name, UE_CVAR_HELP_TEXT_PARAM(Help), Command) {}

	virtual ~FAutoConsoleCommandWithWorldAndArgs() {}
};

/**
 * Autoregistering console command with args and an output device
 */
class FAutoConsoleCommandWithArgsAndOutputDevice : private FAutoConsoleObject
{
	CORE_API FAutoConsoleCommandWithArgsAndOutputDevice(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsAndOutputDeviceDelegate& Command);
public:
	
	FORCEINLINE FAutoConsoleCommandWithArgsAndOutputDevice(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithArgsAndOutputDeviceDelegate& Command, uint32 Flags = ECVF_Default)
		: FAutoConsoleCommandWithArgsAndOutputDevice(Flags, Name, UE_CVAR_HELP_TEXT_PARAM(Help), Command) {}

	virtual ~FAutoConsoleCommandWithArgsAndOutputDevice() {}
};

/**
 * Autoregistering console command with an output device
 */
class FAutoConsoleCommandWithOutputDevice : private FAutoConsoleObject
{
	CORE_API FAutoConsoleCommandWithOutputDevice(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithOutputDeviceDelegate& Command);
public:
	/**
	 * Register a console command that takes an output device
	 *
	 * @param	Name		The name of this command (must not be nullptr)
	 * @param	Help		Help text for this command
	 * @param	Command		The user function to call when this command is executed
	 * @param	Flags		Optional flags bitmask
	 */
	FORCEINLINE FAutoConsoleCommandWithOutputDevice(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithOutputDeviceDelegate& Command, uint32 Flags = ECVF_Default)
		: FAutoConsoleCommandWithOutputDevice(Flags, Name, UE_CVAR_HELP_TEXT_PARAM(Help), Command) {}

	virtual ~FAutoConsoleCommandWithOutputDevice() {}
};

/**
 * Autoregistering console command with world, args, an output device
 */
class FAutoConsoleCommandWithWorldArgsAndOutputDevice : private FAutoConsoleObject
{
	CORE_API FAutoConsoleCommandWithWorldArgsAndOutputDevice(uint32 Flags, const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldArgsAndOutputDeviceDelegate& Command);
public:
	/**
	 * Register a console command that takes an output device
	 *
	 * @param	Name		The name of this command (must not be nullptr)
	 * @param	Help		Help text for this command
	 * @param	Command		The user function to call when this command is executed
	 * @param	Flags		Optional flags bitmask
	 */
	FORCEINLINE FAutoConsoleCommandWithWorldArgsAndOutputDevice(const TCHAR* Name, const TCHAR* Help, const FConsoleCommandWithWorldArgsAndOutputDeviceDelegate& Command, uint32 Flags = ECVF_Default)
		: FAutoConsoleCommandWithWorldArgsAndOutputDevice(Flags, Name, UE_CVAR_HELP_TEXT_PARAM(Help), Command) {}

	virtual ~FAutoConsoleCommandWithWorldArgsAndOutputDevice() {}
};

#else

class FAutoConsoleCommandWithWorld
{
public:
	template<class... Args> FAutoConsoleCommandWithWorld(const Args&...) {}
};

class FAutoConsoleCommandWithWorldAndArgs
{
public:
	template<class... Args> FAutoConsoleCommandWithWorldAndArgs(const Args&...) {}
};

class FAutoConsoleCommandWithOutputDevice
{
public:
	template<class... Args> FAutoConsoleCommandWithOutputDevice(const Args&...) {}
};

class FAutoConsoleCommandWithWorldArgsAndOutputDevice
{
public:
	template<class... Args> FAutoConsoleCommandWithWorldArgsAndOutputDevice(const Args&...) {}
};

#endif


/**
 * A class that can be used to shadow (alias) a CVar, where both names are valid to use.
 * To deprecate a CVar, use FAutoConsoleVariableDeprecated.
 * To deprecate a CCmd, use FAutoConsoleCommandDeprecated.
 * 
 * FAutoConsoleVariableShadow ShadowCVar(TEXT("r.oldname"), TEXT("r.newname"));
 * 
 */
class FAutoConsoleVariableShadow
{
public:

	/**
	 * @param Name The name of this cvar
	 * @param CVarToShadow The name of the existing cvar that this one will shadow
	 * @param LookupFailureBehavior How to handle failure of looking up the CVarToShadow name - defaults to ensure
	 */
	CORE_API FAutoConsoleVariableShadow(const TCHAR* Name, const TCHAR* CVarToShadow, EShadowCVarBehavior LookupFailureBehavior = EShadowCVarBehavior::Ensure);
};

/**
 * A class that can be used to deprecate a CVar - rename your existing CVar to the new name, then
 * create a shadow of it with the old name with something like this:
 *
 * FAutoConsoleVariableDeprecated ShadowCVar(TEXT("r.oldname"), TEXT("r.newname"), TEXT("5.5"));
 * FAutoConsoleVariableDeprecated ShadowCVar(TEXT("r.oldname"), TEXT("r.newname"), TEXT("5.5"), EShadowCVarBehavior::Assert);
 *
 */
class FAutoConsoleVariableDeprecated
{
public:
	/**
	 * @param Name The name of this cvar
	 * @param CVarToShadow The name of the existing cvar that this one will shadow
	 * @param DeprecatedAtVersion THe first UE version when this was deprecated
	 * @param UsageBehavior How to handle uses of the shadowed cvar by the deprecated name - defaults to ensure
	 * @param LookupFailureBehavior How to handle failure of looking up the CVarToShadow name - defaults to ensure
	 * @param DeprecationMessage Optional deprecation message - defaults to empty, auto-generated.
	 */
	CORE_API FAutoConsoleVariableDeprecated(const TCHAR* Name, const TCHAR* CVarToShadow, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior=EShadowCVarBehavior::Ensure, EShadowCVarBehavior LookupFailureBehavior = EShadowCVarBehavior::Ensure, const TCHAR* DeprecationMessage = TEXT(""));
};

/**
 * A class that can be used to deprecate a ConsoleCommand - use just like FAutoConsoleVariableDeprecated
 */
class FAutoConsoleCommandDeprecated
{
public:
	/**
	 * @param Name The name of this cvar
	 * @param CVarToShadow The name of the existing cvar that this one will shadow
	 * @param DeprecatedAtVersion THe first UE version when this was deprecated
	 * @param UsageBehavior How to handle uses of the shadowed cvar by the deprecated name - defaults to ensure
	 * @param LookupFailureBehavior How to handle failure of looking up the CVarToShadow name - defaults to ensure
	 * @param DeprecationMessage Optional deprecation message - defaults to empty, auto-generated.
	 */
	CORE_API FAutoConsoleCommandDeprecated(const TCHAR* Name, const TCHAR* CVarToShadow, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior = EShadowCVarBehavior::Ensure, EShadowCVarBehavior LookupFailureBehavior = EShadowCVarBehavior::Ensure, const TCHAR* DeprecationMessage = TEXT(""));
};

/**
 * A class that can be used to deprecate a cvar, but without making a new one to replace it.
 */
template <typename T>
class TAutoConsoleVariableDeprecated_NoReplace
{
public:
	/**
	 * @param Name The name of this cvar
	 * @param DefaultValue The value this cvar has by default
	 * @param DeprecatedAtVersion The first UE version when this was deprecated
	 * @param UsageBehavior How to handle uses of the deprecated cvar  - defaults to ensure
	 * @param DeprecationMessage Optional deprecation message - defaults to empty, auto-generated.
	 */
	CORE_API TAutoConsoleVariableDeprecated_NoReplace(const TCHAR* Name, const T& DefaultValue, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior=EShadowCVarBehavior::Ensure, const TCHAR* DeprecationMessage = TEXT(""));
};

/**
 * A class that can be used to deprecate a ConsoleCommand, but without making a new one to replace it.
 */
class FAutoConsoleCommandDeprecated_NoReplace
{
public:
	/**
	 * @param Name The name of this command
	 * @param Command The user function to call when this command is executed
	 * @param DeprecatedAtVersion The first UE version when this was deprecated
	 * @param UsageBehavior How to handle uses of the deprecated cvar  - defaults to ensure
	 * @param DeprecationMessage Optional deprecation message - defaults to empty, auto-generated.
	 */
	CORE_API FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior = EShadowCVarBehavior::Ensure, const TCHAR* DeprecationMessage = TEXT(""));
	CORE_API FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const FConsoleCommandDelegate& Command, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior = EShadowCVarBehavior::Ensure, const TCHAR* DeprecationMessage = TEXT(""));
	CORE_API FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const FConsoleCommandWithArgsDelegate& Command, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior = EShadowCVarBehavior::Ensure, const TCHAR* DeprecationMessage = TEXT(""));
	CORE_API FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const FConsoleCommandWithOutputDeviceDelegate& Command, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior = EShadowCVarBehavior::Ensure, const TCHAR* DeprecationMessage = TEXT(""));
	CORE_API FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const FConsoleCommandWithArgsAndOutputDeviceDelegate& Command, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior = EShadowCVarBehavior::Ensure, const TCHAR* DeprecationMessage = TEXT(""));
	CORE_API FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const FConsoleCommandWithWorldDelegate& Command, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior = EShadowCVarBehavior::Ensure, const TCHAR* DeprecationMessage = TEXT(""));
	CORE_API FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const FConsoleCommandWithWorldArgsAndOutputDeviceDelegate& Command, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior = EShadowCVarBehavior::Ensure, const TCHAR* DeprecationMessage = TEXT(""));
	CORE_API FAutoConsoleCommandDeprecated_NoReplace(const TCHAR* Name, const FConsoleCommandWithWorldAndArgsDelegate& Command, const TCHAR* DeprecatedAtVersion, EShadowCVarBehavior UsageBehavior = EShadowCVarBehavior::Ensure, const TCHAR* DeprecationMessage = TEXT(""));
};


/**
 * A class to create a ranged console variable. The min and max can be changed with <name>.min and <name.max>
 */
template<typename T>
class TAutoRangedConsoleVariable : public TAutoConsoleVariable<T>
{
	CORE_API TAutoRangedConsoleVariable(IConsoleVariable* Parent, const TCHAR* Name, T DefaultValue, T MinValue, T MaxValue, const TCHAR* Help, uint32 Flags);
public:
	FORCEINLINE TAutoRangedConsoleVariable(const TCHAR* Name, T DefaultValue, T MinValue, T MaxValue, const TCHAR* Help, uint32 Flags = ECVF_Default, IConsoleVariable* Parent = nullptr)
		: TAutoRangedConsoleVariable(Parent, Name, Forward(DefaultValue), Forward(MinValue), Forward(MaxValue), UE_CVAR_HELP_TEXT_PARAM(Help), Flags) {}
};


CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogConsoleResponse, Log, All);

