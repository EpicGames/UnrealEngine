// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"
#include "Math/UnrealMathUtility.h"
#include "Serialization/Archive.h"
#include "Containers/UnrealString.h"
#include "Misc/Parse.h"
#include "Logging/LogMacros.h"
#include "Misc/DateTime.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/IPlatformFileLogWrapper.h"
#include "Templates/UniquePtr.h"
#include "Async/MappedFileHandle.h"

class IAsyncReadFileHandle;

class FCachedFileHandle : public IFileHandle
{
public:
	FCachedFileHandle(IFileHandle* InFileHandle, bool bInReadable, bool bInWritable)
		: FileHandle(InFileHandle)
		, FilePos(InFileHandle->Tell())
		, TellPos(FilePos)
		, FileSize(InFileHandle->Size())
		, bWritable(bInWritable)
		, bReadable(bInReadable)
		, CurrentCache(0)
	{
		FlushCache();
	}
	
	virtual ~FCachedFileHandle() = default;

	
	virtual int64		Tell() override
	{
		return FilePos;
	}

	virtual bool		Seek(int64 NewPosition) override
	{
		if (NewPosition < 0 || NewPosition > FileSize)
		{
			return false;
		}
		FilePos=NewPosition;
		return true;
	}

	virtual bool		SeekFromEnd(int64 NewPositionRelativeToEnd = 0) override
	{
		return Seek(FileSize - NewPositionRelativeToEnd);
	}

	virtual bool		Read(uint8* Destination, int64 BytesToRead) override
	{
		if (!bReadable || BytesToRead < 0 || (BytesToRead + FilePos > FileSize))
		{
			return false;
		}

		if (BytesToRead == 0)
		{
			return true;
		}

		bool Result = false;
		if (BytesToRead > BufferCacheSize) // reading more than we cache
		{
			// if the file position is within the cache, copy out the remainder of the cache
			int32 CacheIndex=GetCacheIndex(FilePos);
			if (CacheIndex < CacheCount)
			{
				int64 CopyBytes = CacheEnd[CacheIndex]-FilePos;
				FMemory::Memcpy(Destination, BufferCache[CacheIndex]+(FilePos-CacheStart[CacheIndex]), CopyBytes);
				FilePos += CopyBytes;
				BytesToRead -= CopyBytes;
				Destination += CopyBytes;
			}

			if (InnerSeek(FilePos))
			{
				Result = InnerRead(Destination, BytesToRead);
			}
			if (Result)
			{
				FilePos += BytesToRead;
			}
		}
		else
		{
			Result = true;
			
			while (BytesToRead && Result) 
			{
				uint32 CacheIndex=GetCacheIndex(FilePos);
				if (CacheIndex > CacheCount)
				{
					// need to update the cache
					uint64 AlignedFilePos=FilePos&BufferSizeMask; // Aligned Version
					uint64 SizeToRead=FMath::Min<uint64>(BufferCacheSize, FileSize-AlignedFilePos);
					InnerSeek(AlignedFilePos);
					Result = InnerRead(BufferCache[CurrentCache], SizeToRead);

					if (Result)
					{
						CacheStart[CurrentCache] = AlignedFilePos;
						CacheEnd[CurrentCache] = AlignedFilePos+SizeToRead;
						CacheIndex = CurrentCache;
						// move to next cache for update
						CurrentCache++;
						CurrentCache%=CacheCount;
					}
				}

				// copy from the cache to the destination
				if (Result)
				{
					// Analyzer doesn't see this - if this code ever changes make sure there are no buffer overruns!
					CA_ASSUME(CacheIndex < CacheCount);
					uint64 CorrectedBytesToRead=FMath::Min<uint64>(BytesToRead, CacheEnd[CacheIndex]-FilePos);
					FMemory::Memcpy(Destination, BufferCache[CacheIndex]+(FilePos-CacheStart[CacheIndex]), CorrectedBytesToRead);
					FilePos += CorrectedBytesToRead;
					Destination += CorrectedBytesToRead;
					BytesToRead -= CorrectedBytesToRead;
				}
			}
		}
		return Result;
	}

	virtual bool ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset) override
	{
		// concurrent reads won't be cached yet
		return FileHandle->ReadAt(Destination, BytesToRead, Offset);
	}

	virtual bool		Write(const uint8* Source, int64 BytesToWrite) override
	{
		if (!bWritable || BytesToWrite < 0)
		{
			return false;
		}

		if (BytesToWrite == 0)
		{
			return true;
		}

		InnerSeek(FilePos);
		bool Result = FileHandle->Write(Source, BytesToWrite);
		if (Result)
		{
			FilePos += BytesToWrite;
			FileSize = FMath::Max<int64>(FilePos, FileSize);
			FlushCache();
			TellPos = FilePos;
		}
		return Result;
	}

	virtual int64		Size() override
	{
		return FileSize;
	}

	virtual bool		Flush(const bool bFullFlush = false) override
	{
		if (bWritable)
		{
			return FileHandle->Flush(bFullFlush);
		}
		return false;
	}

	virtual bool		Truncate(int64 NewSize) override
	{
		if (bWritable)
		{
			if (FileHandle->Truncate(NewSize))
			{
				FlushCache();
				FilePos = TellPos = FileHandle->Tell();
				FileSize = FileHandle->Size();
				return true;
			}
		}
		return false;
	}

	virtual void ShrinkBuffers() override
	{
		FileHandle->ShrinkBuffers();
	}

private:

	static constexpr uint32 BufferCacheSize = 64 * 1024; // Seems to be the magic number for best perf
	static constexpr uint64 BufferSizeMask  = ~((uint64)BufferCacheSize-1);
	static constexpr uint32	CacheCount		= 2;

	bool InnerSeek(uint64 Pos)
	{
		if (Pos==TellPos) 
		{
			return true;
		}
		bool bOk=FileHandle->Seek(Pos);
		if (bOk)
		{
			TellPos=Pos;
		}
		return bOk;
	}
	bool InnerRead(uint8* Dest, uint64 BytesToRead)
	{
		if (FileHandle->Read(Dest, BytesToRead))
		{
			TellPos += BytesToRead;
			return true;
		}
		return false;
	}
	int32 GetCacheIndex(int64 Pos) const
	{
		for (uint32 i=0; i<UE_ARRAY_COUNT(CacheStart); ++i)
		{
			if (Pos >= CacheStart[i] && Pos < CacheEnd[i]) 
			{
				return i;
			}
		}
		return CacheCount+1;
	}
	void FlushCache() 
	{
		for (uint32 i=0; i<CacheCount; ++i)
		{
			CacheStart[i] = CacheEnd[i] = -1;
		}
	}

	TUniquePtr<IFileHandle>	FileHandle;
	int64					FilePos; /* Desired position in the file stream, this can be different to FilePos due to the cache */
	int64					TellPos; /* Actual position in the file,  this can be different to FilePos */
	int64					FileSize;
	bool					bWritable;
	bool					bReadable;
	uint8					BufferCache[CacheCount][BufferCacheSize];
	int64					CacheStart[CacheCount];
	int64					CacheEnd[CacheCount];
	int32					CurrentCache;
};

class FCachedReadPlatformFile : public IPlatformFile
{
	IPlatformFile*		LowerLevel;
public:
	static const TCHAR* GetTypeName()
	{
		return TEXT("CachedReadFile");
	}

	FCachedReadPlatformFile()
		: LowerLevel(nullptr)
	{
	}

	//~ For visibility of overloads we don't override
	using IPlatformFile::IterateDirectory;
	using IPlatformFile::IterateDirectoryRecursively;
	using IPlatformFile::IterateDirectoryStat;
	using IPlatformFile::IterateDirectoryStatRecursively;

	virtual bool Initialize(IPlatformFile* Inner, const TCHAR* CommandLineParam) override
	{
		// Inner is required.
		check(Inner != nullptr);
		LowerLevel = Inner;
		return !!LowerLevel;
	}
	virtual bool ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const override
	{
#ifndef PLATFORM_PROVIDES_FILE_CACHE
#define PLATFORM_PROVIDES_FILE_CACHE 0
#endif
		// Default to false on platforms that already do platform file level caching
		bool bPlatformProvidesFileCache = PLATFORM_PROVIDES_FILE_CACHE;
		bool bPlatformWindows = PLATFORM_WINDOWS;
		bool bResult = !bPlatformProvidesFileCache && !bPlatformWindows && FPlatformProperties::RequiresCookedData();

		// Allow a choice between shorter load times or less memory on desktop platforms.
		// Note: this cannot be in config since they aren't read at that point.
#if (PLATFORM_DESKTOP || PLATFORM_PROVIDES_FILE_CACHE)
		{
			if (FParse::Param(CmdLine, TEXT("NoCachedReadFile")))
			{
				bResult = false;
			}
			else if (FParse::Param(CmdLine, TEXT("CachedReadFile")))
			{
				bResult = true;
			}

			UE_LOGF(LogPlatformFile, Verbose, "%ls cached read wrapper", bResult ? TEXT("Using") : TEXT("Not using"));
		}
#endif
		return bResult;
	}
	IPlatformFile* GetLowerLevel() override
	{
		return LowerLevel;
	}
	virtual void SetLowerLevel(IPlatformFile* NewLowerLevel) override
	{
		LowerLevel = NewLowerLevel;
	}
	virtual const TCHAR* GetName() const override
	{
		return FCachedReadPlatformFile::GetTypeName();
	}
	virtual bool		FileExists(const TCHAR* Filename) override
	{
		return LowerLevel->FileExists(Filename);
	}
	virtual int64		FileSize(const TCHAR* Filename) override
	{
		return LowerLevel->FileSize(Filename);
	}
	virtual bool		DeleteFile(const TCHAR* Filename) override
	{
		return LowerLevel->DeleteFile(Filename);
	}
	virtual bool		IsReadOnly(const TCHAR* Filename) override
	{
		return LowerLevel->IsReadOnly(Filename);
	}
	virtual bool		MoveFile(const TCHAR* To, const TCHAR* From) override
	{
		return LowerLevel->MoveFile(To, From);
	}
	virtual bool		SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override
	{
		return LowerLevel->SetReadOnly(Filename, bNewReadOnlyValue);
	}
	virtual FDateTime	GetTimeStamp(const TCHAR* Filename) override
	{
		return LowerLevel->GetTimeStamp(Filename);
	}
	virtual void		SetTimeStamp(const TCHAR* Filename, FDateTime DateTime) override
	{
		LowerLevel->SetTimeStamp(Filename, DateTime);
	}
	virtual FDateTime	GetAccessTimeStamp(const TCHAR* Filename) override
	{
		return LowerLevel->GetAccessTimeStamp(Filename);
	}
	virtual FString	GetFilenameOnDisk(const TCHAR* Filename) override
	{
		return LowerLevel->GetFilenameOnDisk(Filename);
	}
	virtual IFileHandle*	OpenRead(const TCHAR* Filename, bool bAllowWrite) override
	{
		IFileHandle* InnerHandle=LowerLevel->OpenRead(Filename, bAllowWrite);
		if (!InnerHandle)
		{
			return nullptr;
		}
		return new FCachedFileHandle(InnerHandle, true, false);
	}
	virtual IFileHandle*	OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override
	{
		IFileHandle* InnerHandle=LowerLevel->OpenWrite(Filename, bAppend, bAllowRead);
		if (!InnerHandle)
		{
			return nullptr;
		}
		return new FCachedFileHandle(InnerHandle, bAllowRead, true);
	}
	virtual bool		DirectoryExists(const TCHAR* Directory) override
	{
		return LowerLevel->DirectoryExists(Directory);
	}
	virtual bool		CreateDirectory(const TCHAR* Directory) override
	{
		return LowerLevel->CreateDirectory(Directory);
	}
	virtual bool		DeleteDirectory(const TCHAR* Directory) override
	{
		return LowerLevel->DeleteDirectory(Directory);
	}
	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override
	{
		return LowerLevel->GetStatData(FilenameOrDirectory);
	}
	virtual bool		IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override
	{
		return LowerLevel->IterateDirectory(Directory, Visitor);
	}
	virtual bool		IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override
	{
		return LowerLevel->IterateDirectoryRecursively(Directory, Visitor);
	}
	virtual bool		IterateDirectoryStat(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) override
	{
		return LowerLevel->IterateDirectoryStat(Directory, Visitor);
	}
	virtual bool		IterateDirectoryStatRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) override
	{
		return LowerLevel->IterateDirectoryStatRecursively(Directory, Visitor);
	}
	virtual void		FindFiles(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension)
	{
		return LowerLevel->FindFiles(FoundFiles, Directory, FileExtension);
	}
	virtual void		FindFilesRecursively(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension)
	{
		return LowerLevel->FindFilesRecursively(FoundFiles, Directory, FileExtension);
	}
	virtual bool		DeleteDirectoryRecursively(const TCHAR* Directory) override
	{
		return LowerLevel->DeleteDirectoryRecursively(Directory);
	}
	virtual bool		CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags = EPlatformFileRead::None, EPlatformFileWrite WriteFlags = EPlatformFileWrite::None) override
	{
		return LowerLevel->CopyFile(To, From, ReadFlags, WriteFlags);
	}
	virtual bool		CreateDirectoryTree(const TCHAR* Directory) override
	{
		return LowerLevel->CreateDirectoryTree(Directory);
	}
	virtual bool		CopyDirectoryTree(const TCHAR* DestinationDirectory, const TCHAR* Source, bool bOverwriteAllExisting) override
	{
		return LowerLevel->CopyDirectoryTree(DestinationDirectory, Source, bOverwriteAllExisting);
	}
	virtual FString		ConvertToAbsolutePathForExternalAppForRead( const TCHAR* Filename ) override
	{
		return LowerLevel->ConvertToAbsolutePathForExternalAppForRead(Filename);
	}
	virtual FString		ConvertToAbsolutePathForExternalAppForWrite( const TCHAR* Filename ) override
	{
		return LowerLevel->ConvertToAbsolutePathForExternalAppForWrite(Filename);
	}
	virtual bool		SendMessageToServer(const TCHAR* Message, IFileServerMessageHandler* Handler) override
	{
		return LowerLevel->SendMessageToServer(Message, Handler);
	}
	virtual IAsyncReadFileHandle* OpenAsyncRead(const TCHAR* Filename, bool bAllowWrite = false) override
	{
		return LowerLevel->OpenAsyncRead(Filename, bAllowWrite);
	}
	virtual FOpenMappedResult OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions = EOpenReadFlags::None, int64 MaximumSize = 0) override
	{
		return LowerLevel->OpenMappedEx(Filename, OpenOptions, MaximumSize);
	}
	virtual FOpenMappedResult OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions = EOpenMappedFlags::None, int64 MaximumSize = 0) override
	{
		return LowerLevel->OpenMappedEx2(Filename, OpenOptions, MaximumSize);
	}
	virtual void SetAsyncMinimumPriority(EAsyncIOPriorityAndFlags MinPriority) override
	{
		LowerLevel->SetAsyncMinimumPriority(MinPriority);
	}
	virtual bool		FileJournalIsAvailable(const TCHAR* VolumeOrPath = nullptr,
		ELogVerbosity::Type* OutErrorLevel = nullptr, FString* OutError = nullptr) override
	{
		return LowerLevel->FileJournalIsAvailable(VolumeOrPath, OutErrorLevel, OutError);
	}
	virtual bool		FileJournalIterateDirectory(const TCHAR* Directory,
		FDirectoryJournalVisitorFunc Visitor, FString* OutError = nullptr) override
	{
		return LowerLevel->FileJournalIterateDirectory(Directory, Visitor, OutError);
	}
	virtual EFileJournalResult FileJournalGetLatestEntry(const TCHAR* VolumeName, FFileJournalId& OutJournalId,
		FFileJournalEntryHandle& OutEntryHandle, FString* OutError = nullptr) override
	{
		return LowerLevel->FileJournalGetLatestEntry(VolumeName, OutJournalId, OutEntryHandle, OutError);
	}
	virtual EFileJournalResult FileJournalReadModified(const TCHAR* VolumeName,
		const FFileJournalId& JournalIdOfStartingEntry, const FFileJournalEntryHandle& StartingJournalEntry,
		TMap<FFileJournalFileHandle, FString>& KnownDirectories, TSet<FString>& OutModifiedDirectories,
		FFileJournalEntryHandle& OutNextJournalEntry, FString* OutError = nullptr) override
	{
		return LowerLevel->FileJournalReadModified(VolumeName, JournalIdOfStartingEntry, StartingJournalEntry,
			KnownDirectories, OutModifiedDirectories, OutNextJournalEntry, OutError);
	}
	virtual FFileJournalData FileJournalGetFileData(const TCHAR* FilenameOrDirectory,
		FString* OutError = nullptr) override
	{
		return LowerLevel->FileJournalGetFileData(FilenameOrDirectory, OutError);
	}
	virtual FString		FileJournalGetVolumeName(FStringView InPath) override
	{
		return LowerLevel->FileJournalGetVolumeName(InPath);
	}
	virtual uint64		FileJournalGetMaximumSize(const TCHAR* VolumeOrPath = nullptr,
		ELogVerbosity::Type* OutErrorLevel = nullptr, FString* OutError = nullptr) const override
	{
		return LowerLevel->FileJournalGetMaximumSize(VolumeOrPath, OutErrorLevel, OutError);
	}
};
