// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformTime.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Misc/DateTime.h"
#include "Misc/ScopeLock.h"
#include "Templates/UniquePtr.h"
#include "Trace/Detail/Channel.h"

class FLoggedPlatformFile;
class FOutputDevice;
class IAsyncReadFileHandle;
class IMappedFileHandle;

/**
 * Wrapper to log the low level file system
**/
DECLARE_LOG_CATEGORY_EXTERN(LogPlatformFile, Log, All);

extern bool bSuppressFileLog;

#define FILE_LOG(CategoryName, Verbosity, Format, ...) \
	if (!bSuppressFileLog) \
	{ \
		bSuppressFileLog = true; \
		UE_LOG(CategoryName, Verbosity, Format, ##__VA_ARGS__); \
		bSuppressFileLog = false; \
	}

class FLoggedPlatformFile;

class FLoggedFileHandle : public IFileHandle
{
	TUniquePtr<IFileHandle>	FileHandle;
	FString					Filename;
#if !UE_BUILD_SHIPPING
	FLoggedPlatformFile& PlatformFile;
#endif
public:

	CORE_API FLoggedFileHandle(IFileHandle* InFileHandle, const TCHAR* InFilename, FLoggedPlatformFile& InOwner);
	CORE_API virtual ~FLoggedFileHandle();

	virtual int64		Tell() override
	{
		FILE_LOG(LogPlatformFile, VeryVerbose, TEXT("Tell %s"), *Filename);
		double StartTime = FPlatformTime::Seconds();
		int64 Result = FileHandle->Tell();
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, VeryVerbose, TEXT("Tell return %lld [%fms]"), Result, ThisTime);
		return Result;
	}
	virtual bool		Seek(int64 NewPosition) override
	{
		FILE_LOG(LogPlatformFile, VeryVerbose, TEXT("Seek %s %lld"), *Filename, NewPosition);
		double StartTime = FPlatformTime::Seconds();
		bool Result = FileHandle->Seek(NewPosition);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, VeryVerbose, TEXT("Seek return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual bool		SeekFromEnd(int64 NewPositionRelativeToEnd) override
	{
		FILE_LOG(LogPlatformFile, VeryVerbose, TEXT("SeekFromEnd %s %lld"), *Filename, NewPositionRelativeToEnd);
		double StartTime = FPlatformTime::Seconds();
		bool Result = FileHandle->SeekFromEnd(NewPositionRelativeToEnd);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, VeryVerbose, TEXT("SeekFromEnd return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual bool		Read(uint8* Destination, int64 BytesToRead) override
	{
		FILE_LOG(LogPlatformFile, VeryVerbose, TEXT("Read %s %lld"), *Filename, BytesToRead);
		double StartTime = FPlatformTime::Seconds();
		bool Result = FileHandle->Read(Destination, BytesToRead);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, VeryVerbose, TEXT("Read return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual bool		ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset) override
	{
		FILE_LOG(LogPlatformFile, VeryVerbose, TEXT("ReadAt %s %lld %lld"), *Filename, BytesToRead, Offset);
		double StartTime = FPlatformTime::Seconds();
		bool Result = FileHandle->ReadAt(Destination, BytesToRead, Offset);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, VeryVerbose, TEXT("ReadAt return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual bool		Write(const uint8* Source, int64 BytesToWrite) override
	{
		FILE_LOG(LogPlatformFile, VeryVerbose, TEXT("Write %s %lld"), *Filename, BytesToWrite);
		double StartTime = FPlatformTime::Seconds();
		bool Result = FileHandle->Write(Source, BytesToWrite);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, VeryVerbose, TEXT("Write return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual int64		Size() override
	{
		FILE_LOG(LogPlatformFile, Verbose, TEXT("Size %s"), *Filename);
		double StartTime = FPlatformTime::Seconds();
		int64 Result = FileHandle->Size();
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Verbose, TEXT("Size return %lld [%fms]"), Result, ThisTime);
		return Result;
	}
	virtual bool		Flush(const bool bFullFlush = false) override
	{
		FILE_LOG(LogPlatformFile, Verbose, TEXT("Flush %s %s"), *Filename, (bFullFlush ? TEXT("full") : TEXT("partial")));
		double StartTime = FPlatformTime::Seconds();
		bool bResult = FileHandle->Flush(bFullFlush);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Verbose, TEXT("Flush return %d [%fms]"), int32(bResult), ThisTime);
		return bResult;
	}
	virtual bool		Truncate(int64 NewSize) override
	{
		FILE_LOG(LogPlatformFile, Verbose, TEXT("Truncate %s %lld"), *Filename, NewSize);
		double StartTime = FPlatformTime::Seconds();
		bool bResult = FileHandle->Truncate(NewSize);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Verbose, TEXT("Truncate return %d [%fms]"), int32(bResult), ThisTime);
		return bResult;
	}
	virtual void ShrinkBuffers() override
	{
		FILE_LOG(LogPlatformFile, Verbose, TEXT("ShrinkBuffers %s"), *Filename);
		double StartTime = FPlatformTime::Seconds();
		FileHandle->ShrinkBuffers();
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Verbose, TEXT("ShrinkBuffers return [%fms]"), ThisTime);
	}
};

class FLoggedPlatformFile : public IPlatformFile
{
	IPlatformFile* LowerLevel;

#if !UE_BUILD_SHIPPING
	FCriticalSection LogFileCritical;
	TMap<FString, int32> OpenHandles;
#endif

public:
	static const TCHAR* GetTypeName()
	{
		return TEXT("LogFile");
	}

	FLoggedPlatformFile()
		: LowerLevel(nullptr)
	{
	}

	//~ For visibility of overloads we don't override
	using IPlatformFile::IterateDirectory;
	using IPlatformFile::IterateDirectoryRecursively;
	using IPlatformFile::IterateDirectoryStat;
	using IPlatformFile::IterateDirectoryStatRecursively;

	CORE_API virtual bool ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const override;

	CORE_API virtual bool Initialize(IPlatformFile* Inner, const TCHAR* CommandLineParam) override;

	IPlatformFile* GetLowerLevel() override
	{
		return LowerLevel;
	}
	virtual void SetLowerLevel(IPlatformFile* NewLowerLevel) override
	{
		LowerLevel = NewLowerLevel;
	}
	virtual const TCHAR* GetName() const override
	{
		return FLoggedPlatformFile::GetTypeName();
	}

	virtual bool		FileExists(const TCHAR* Filename) override
	{
		FString DataStr = FString::Printf(TEXT("FileExists %s"), Filename);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		bool Result = LowerLevel->FileExists(Filename);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("FileExists return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual int64		FileSize(const TCHAR* Filename) override
	{
		FString DataStr = FString::Printf(TEXT("FileSize %s"), Filename);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		int64 Result = LowerLevel->FileSize(Filename);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("FileSize return %lld [%fms]"), Result, ThisTime);
		return Result;
	}
	virtual bool		DeleteFile(const TCHAR* Filename) override
	{
		FString DataStr = FString::Printf(TEXT("DeleteFile %s"), Filename);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		bool Result = LowerLevel->DeleteFile(Filename);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("DeleteFile return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual bool		IsReadOnly(const TCHAR* Filename) override
	{
		FString DataStr = FString::Printf(TEXT("IsReadOnly %s"), Filename);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		bool Result = LowerLevel->IsReadOnly(Filename);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("IsReadOnly return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual bool		MoveFile(const TCHAR* To, const TCHAR* From) override
	{
		FString DataStr = FString::Printf(TEXT("MoveFile %s %s"), To, From);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		bool Result = LowerLevel->MoveFile(To, From);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("MoveFile return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual bool		SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override
	{
		FString DataStr = FString::Printf(TEXT("SetReadOnly %s %d"), Filename, int32(bNewReadOnlyValue));
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		bool Result = LowerLevel->SetReadOnly(Filename, bNewReadOnlyValue);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("SetReadOnly return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual FDateTime	GetTimeStamp(const TCHAR* Filename) override
	{
		FString DataStr = FString::Printf(TEXT("GetTimeStamp %s"), Filename);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		FDateTime Result = LowerLevel->GetTimeStamp(Filename);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("GetTimeStamp return %llx [%fms]"), Result.GetTicks(), ThisTime);
		return Result;
	}
	virtual void		SetTimeStamp(const TCHAR* Filename, FDateTime DateTime) override
	{
		FString DataStr = FString::Printf(TEXT("SetTimeStamp %s"), Filename);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		LowerLevel->SetTimeStamp(Filename, DateTime);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("SetTimeStamp [%fms]"), ThisTime);
	}
	virtual FDateTime	GetAccessTimeStamp(const TCHAR* Filename) override
	{
		FString DataStr = FString::Printf(TEXT("GetAccessTimeStamp %s"), Filename);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		FDateTime Result = LowerLevel->GetAccessTimeStamp(Filename);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("GetAccessTimeStamp return %llx [%fms]"), Result.GetTicks(), ThisTime);
		return Result;
	}
	virtual FString	GetFilenameOnDisk(const TCHAR* Filename) override
	{
		FString DataStr = FString::Printf(TEXT("GetFilenameOnDisk %s"), Filename);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		FString Result = LowerLevel->GetFilenameOnDisk(Filename);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("GetFilenameOnDisk return %s [%fms]"), *Result, ThisTime);
		return Result;
	}
	virtual IFileHandle*	OpenRead(const TCHAR* Filename, bool bAllowWrite) override
	{
		FString DataStr = FString::Printf(TEXT("OpenRead %s"), Filename);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		IFileHandle* Result = LowerLevel->OpenRead(Filename, bAllowWrite);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("OpenRead return %llx [%fms]"), uint64(Result), ThisTime);
		return Result ? (new FLoggedFileHandle(Result, Filename, *this)) : Result;
	}
	virtual IFileHandle*	OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override
	{
		FString DataStr = FString::Printf(TEXT("OpenWrite %s %d %d"), Filename, int32(bAppend), int32(bAllowRead));
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		IFileHandle* Result = LowerLevel->OpenWrite(Filename, bAppend, bAllowRead);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("OpenWrite return %llx [%fms]"), uint64(Result), ThisTime);
		return Result ? (new FLoggedFileHandle(Result, Filename, *this)) : Result;
	}

	virtual bool		DirectoryExists(const TCHAR* Directory) override
	{
		FString DataStr = FString::Printf(TEXT("DirectoryExists %s"), Directory);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		bool Result = LowerLevel->DirectoryExists(Directory);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("DirectoryExists return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual bool		CreateDirectory(const TCHAR* Directory) override
	{
		FString DataStr = FString::Printf(TEXT("CreateDirectory %s"), Directory);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		bool Result = LowerLevel->CreateDirectory(Directory);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("CreateDirectory return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual bool		DeleteDirectory(const TCHAR* Directory) override
	{
		FString DataStr = FString::Printf(TEXT("DeleteDirectory %s"), Directory);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		bool Result = LowerLevel->DeleteDirectory(Directory);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("DeleteDirectory return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override
	{
		FString DataStr = FString::Printf(TEXT("GetStatData %s"), FilenameOrDirectory);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		FFileStatData Result = LowerLevel->GetStatData(FilenameOrDirectory);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("GetStatData return %d [%fms]"), int32(Result.bIsValid), ThisTime);
		return Result;
	}

	class FLogVisitor : public IPlatformFile::FDirectoryVisitor
	{
	public:
		FDirectoryVisitor&	Visitor;
		FLogVisitor(FDirectoryVisitor& InVisitor)
			: Visitor(InVisitor)
		{
		}
		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			FILE_LOG(LogPlatformFile, Verbose, TEXT("Visit %s %d"), FilenameOrDirectory, int32(bIsDirectory));
			double StartTime = FPlatformTime::Seconds();
			bool Result = Visitor.CallShouldVisitAndVisit(FilenameOrDirectory, bIsDirectory);
			double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
			FILE_LOG(LogPlatformFile, Verbose, TEXT("Visit return %d [%fms]"), int32(Result), ThisTime);
			return Result;
		}
	};

	virtual bool		IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override
	{
		FString DataStr = FString::Printf(TEXT("IterateDirectory %s"), Directory);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		FLogVisitor LogVisitor(Visitor);
		bool Result = LowerLevel->IterateDirectory(Directory, LogVisitor);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("IterateDirectory return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual bool		IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override
	{
		FString DataStr = FString::Printf(TEXT("IterateDirectoryRecursively %s"), Directory);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		FLogVisitor LogVisitor(Visitor);
		bool Result = LowerLevel->IterateDirectoryRecursively(Directory, LogVisitor);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("IterateDirectoryRecursively return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}

	class FLogStatVisitor : public IPlatformFile::FDirectoryStatVisitor
	{
	public:
		FDirectoryStatVisitor&	Visitor;
		FLogStatVisitor(FDirectoryStatVisitor& InVisitor)
			: Visitor(InVisitor)
		{
		}
		virtual bool Visit(const TCHAR* FilenameOrDirectory, const FFileStatData& StatData) override
		{
			FILE_LOG(LogPlatformFile, Verbose, TEXT("Visit %s %d"), FilenameOrDirectory, int32(StatData.bIsDirectory));
			double StartTime = FPlatformTime::Seconds();
			bool Result = Visitor.CallShouldVisitAndVisit(FilenameOrDirectory, StatData);
			double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
			FILE_LOG(LogPlatformFile, Verbose, TEXT("Visit return %d [%fms]"), int32(Result), ThisTime);
			return Result;
		}
	};

	virtual bool		IterateDirectoryStat(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) override
	{
		FString DataStr = FString::Printf(TEXT("IterateDirectoryStat %s"), Directory);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		FLogStatVisitor LogVisitor(Visitor);
		bool Result = LowerLevel->IterateDirectoryStat(Directory, LogVisitor);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("IterateDirectoryStat return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual bool		IterateDirectoryStatRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) override
	{
		FString DataStr = FString::Printf(TEXT("IterateDirectoryStatRecursively %s"), Directory);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		FLogStatVisitor LogVisitor(Visitor);
		bool Result = LowerLevel->IterateDirectoryStatRecursively(Directory, LogVisitor);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("IterateDirectoryStatRecursively return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}

	virtual bool		DeleteDirectoryRecursively(const TCHAR* Directory) override
	{
		FString DataStr = FString::Printf(TEXT("DeleteDirectoryRecursively %s"), Directory);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		bool Result = LowerLevel->DeleteDirectoryRecursively(Directory);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("DeleteDirectoryRecursively return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}
	virtual bool		CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags = EPlatformFileRead::None, EPlatformFileWrite WriteFlags = EPlatformFileWrite::None) override
	{
		FString DataStr = FString::Printf(TEXT("CopyFile %s %s"), To, From);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		bool Result = LowerLevel->CopyFile(To, From, ReadFlags, WriteFlags);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("CopyFile return %d [%fms]"), int32(Result), ThisTime);
		return Result;
	}

#if !UE_BUILD_SHIPPING
	void OnHandleOpen(const FString& Filename)
	{
		FScopeLock LogFileLock(&LogFileCritical);
		int32& NumOpenHandles = OpenHandles.FindOrAdd(Filename);
		NumOpenHandles++;
	}
	void OnHandleClosed(const FString& Filename)
	{
		FScopeLock LogFileLock(&LogFileCritical);
		int32& NumOpenHandles = OpenHandles.FindChecked(Filename);
		if (--NumOpenHandles == 0)
		{
			OpenHandles.Remove(Filename);
		}
	}
	CORE_API void HandleDumpCommand(const TCHAR* Cmd, FOutputDevice& Ar);
#endif
	virtual IAsyncReadFileHandle* OpenAsyncRead(const TCHAR* Filename, bool bAllowWrite = false) override
	{
		FString DataStr = FString::Printf(TEXT("OpenAsyncRead %s"), Filename);
		SCOPED_NAMED_EVENT_FSTRING(DataStr, FColor::Emerald);
		FILE_LOG(LogPlatformFile, Log, TEXT("%s"), *DataStr);
		double StartTime = FPlatformTime::Seconds();
		IAsyncReadFileHandle* Result = LowerLevel->OpenAsyncRead(Filename, bAllowWrite);
		double ThisTime = (FPlatformTime::Seconds() - StartTime) / 1000.0;
		FILE_LOG(LogPlatformFile, Log, TEXT("OpenAsyncRead return %llx [%fms]"), uint64(Result), ThisTime);
		//@todo no wrapped logging for async file handles (yet)
		return Result;
	}
	virtual FOpenMappedResult OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions = EOpenReadFlags::None, int64 MaximumSize = 0) override;
	virtual FOpenMappedResult OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions = EOpenMappedFlags::None, int64 MaximumSize = 0) override;

	virtual void SetAsyncMinimumPriority(EAsyncIOPriorityAndFlags Priority) override
	{
		LowerLevel->SetAsyncMinimumPriority(Priority);
	}
	virtual bool		FileJournalIsAvailable(const TCHAR* VolumeOrPath = nullptr,
		ELogVerbosity::Type* OutErrorLevel = nullptr, FString* OutError = nullptr) override
	{
		return LowerLevel->FileJournalIsAvailable(VolumeOrPath, OutErrorLevel, OutError);
	}
	virtual bool		FileJournalIterateDirectory(const TCHAR* Directory,
		FDirectoryJournalVisitorFunc Visitor, FString* OutError = nullptr) override
	{
		return LowerLevel->FileJournalIterateDirectory(Directory, Visitor, OutError);
	}
	virtual EFileJournalResult FileJournalGetLatestEntry(const TCHAR* VolumeName, FFileJournalId& OutJournalId,
		FFileJournalEntryHandle& OutEntryHandle, FString* OutError = nullptr) override
	{
		return LowerLevel->FileJournalGetLatestEntry(VolumeName, OutJournalId, OutEntryHandle, OutError);
	}
	virtual EFileJournalResult FileJournalReadModified(const TCHAR* VolumeName,
		const FFileJournalId& JournalIdOfStartingEntry, const FFileJournalEntryHandle& StartingJournalEntry,
		TMap<FFileJournalFileHandle, FString>& KnownDirectories, TSet<FString>& OutModifiedDirectories,
		FFileJournalEntryHandle& OutNextJournalEntry, FString* OutError = nullptr) override
	{
		return LowerLevel->FileJournalReadModified(VolumeName, JournalIdOfStartingEntry, StartingJournalEntry,
			KnownDirectories, OutModifiedDirectories, OutNextJournalEntry, OutError);
	}
	virtual FFileJournalData FileJournalGetFileData(const TCHAR* FilenameOrDirectory,
		FString* OutError = nullptr) override
	{
		return LowerLevel->FileJournalGetFileData(FilenameOrDirectory, OutError);
	}
	virtual FString		FileJournalGetVolumeName(FStringView InPath) override
	{
		return LowerLevel->FileJournalGetVolumeName(InPath);
	}
	virtual uint64		FileJournalGetMaximumSize(const TCHAR* VolumeOrPath = nullptr,
		ELogVerbosity::Type* OutErrorLevel = nullptr, FString* OutError = nullptr) const override
	{
		return LowerLevel->FileJournalGetMaximumSize(VolumeOrPath, OutErrorLevel, OutError);
	}
};
