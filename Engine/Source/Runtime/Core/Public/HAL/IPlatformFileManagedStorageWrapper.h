// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Find.h"
#include "Algo/IndexOf.h"
#include "Async/Async.h"
#include "Async/MappedFileHandle.h"
#include "Async/TaskGraphInterfaces.h"
#include "Concepts/DerivedFrom.h"
#include "Concepts/SameAs.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformString.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"

#include <atomic>

CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogPlatformFileManagedStorage, Log, All);

namespace ManagedStorageInternal
{
	// Same as FPaths::IsUnderDirectory, but assume the paths are already full
	// Also is *always* case insensitive since we are concerned with filtering and 
	// not whether a directory actually exists.
	CORE_API bool IsUnderDirectory(const FString& InPath, const FString& InDirectory);
}

struct FPersistentManagedFile
{
	FString FullFilename;
	int32 Category = INDEX_NONE;

	bool IsValid() const { return Category >= 0; }
	explicit operator bool() const { return Category >= 0; }

	// Helpers to avoid unnecessarily attempting to manage multiple times, use empty string as sentinel
	bool HasFailedTryToManage() const { return FullFilename.IsEmpty(); }
	void SetHasFailedTryToManage() { FullFilename.Empty(); }

	void Clear()
	{
		Category = INDEX_NONE;
	}
};

class FManagedStorageFileLockRegistry
{
private:
	friend class FManagedStorageScopeFileLock;

	FCriticalSection* GetLock(const FString& InFilename)
	{
		uint32 KeyHash = GetTypeHash(InFilename);

		FScopeLock Lock(&FileLockMapCS);

		FLockData& LockData = FileLockMap.FindOrAddByHash(KeyHash, InFilename);
		++LockData.RefCount;

		return LockData.CS.Get();
	}

	void ReleaseLock(const FString& InFilename)
	{
		uint32 KeyHash = GetTypeHash(InFilename);

		FScopeLock Lock(&FileLockMapCS);

		FLockData* LockData = FileLockMap.FindByHash(KeyHash, InFilename);
		check(LockData);

		--LockData->RefCount;

		if (LockData->RefCount == 0)
		{
			FileLockMap.RemoveByHash(KeyHash, InFilename);
		}
	}

public:
	FCriticalSection* GetRegistryLock() const
	{
		return &FileLockMapCS;
	}

	void ForEachFile(const UE::FDeferLock& SkipLock, TFunctionRef<void(const FString&)> Function) const
	{
		for (const TPair<FString, FLockData>& Pair : FileLockMap)
		{
			Function(Pair.Key);
		}
	}

	void ForEachFile(TFunctionRef<void(const FString&)> Function) const
	{
		FScopeLock Lock(&FileLockMapCS);
		ForEachFile(UE::DeferLock, Function);
	}

private:

	struct FLockData
	{
		TUniquePtr<FCriticalSection> CS{ MakeUnique<FCriticalSection>() };
		int32 RefCount = 0;
	};

	TMap<FString, FLockData> FileLockMap;
	mutable FCriticalSection FileLockMapCS;
};

class FManagedStorageScopeFileLock
{
public:
	explicit FManagedStorageScopeFileLock(FPersistentManagedFile InManagedFile)
		: ManagedFile(MoveTemp(InManagedFile))
	{
		Lock();
	}

	~FManagedStorageScopeFileLock()
	{
		Unlock();
	}

	FManagedStorageScopeFileLock(const FManagedStorageScopeFileLock&) = delete;
	FManagedStorageScopeFileLock& operator=(const FManagedStorageScopeFileLock&) = delete;

	FManagedStorageScopeFileLock(FManagedStorageScopeFileLock&& InOther) = delete;
	FManagedStorageScopeFileLock& operator=(FManagedStorageScopeFileLock&& InOther) = delete;
	
	CORE_API void Unlock();

private:
	CORE_API void Lock();

private:
	FPersistentManagedFile ManagedFile;
	FCriticalSection* pFileCS = nullptr;
};

enum class EPersistentStorageManagerFileSizeFlags : uint8
{
	None = 0,
	OnlyUpdateIfLess = (1 << 0),
	RespectQuota = (1 << 1)
};
ENUM_CLASS_FLAGS(EPersistentStorageManagerFileSizeFlags);

struct FPersistentStorageCategory
{
public:
	FPersistentStorageCategory(FString InCategoryName, TArray<FString> InDirectories, const int64 InQuota, const int64 InOptionalQuota)
		: CategoryName(MoveTemp(InCategoryName))
		, Directories(MoveTemp(InDirectories))
		, StorageQuota(InQuota)
		, OptionalStorageQuota(InOptionalQuota)
	{
		check(OptionalStorageQuota < StorageQuota || StorageQuota <= 0); // optional storage quota should be a subset of the total storage quota
	}

	const FString& GetCategoryName() const
	{
		return CategoryName;
	}

	int64 GetCategoryQuota() const
	{
		return StorageQuota;
	}

	int64 GetCategoryOptionalQuota() const
	{
		return OptionalStorageQuota;
	}

	int64 GetUsedSize() const
	{
		return UsedQuota;
	}

	int64 GetAvailableSize() const
	{
		int64 ActualStorageQuota = StorageQuota;
		if (ActualStorageQuota < 0)
		{
			ActualStorageQuota = MAX_int64;
		}
		return ActualStorageQuota - UsedQuota;
	}

	bool IsInCategory(const FString& Path) const
	{
		return ShouldManageFile(Path);
	}

	bool IsCategoryFull() const
	{
		return GetAvailableSize() <= 0;
	}

	EPersistentStorageManagerFileSizeFlags AddOrUpdateFile(const FString& Filename, const int64 FileSize, EPersistentStorageManagerFileSizeFlags Flags)
	{
		uint32 KeyHash = GetTypeHash(Filename);

		int64 OldFileSize = 0;
		{
			FReadScopeLock ScopeLock(FileSizesLock);
			int64* pOldFileSize = FileSizes.FindByHash(KeyHash, Filename);
			if (pOldFileSize)
			{
				OldFileSize = *pOldFileSize;
			}
		}

		EPersistentStorageManagerFileSizeFlags Result = TryUpdateQuota(OldFileSize, FileSize, Flags);
		if (Result == EPersistentStorageManagerFileSizeFlags::None)
		{
			FWriteScopeLock ScopeLock(FileSizesLock);
			FileSizes.AddByHash(KeyHash, Filename, FileSize);

			UE_LOGF(LogPlatformFileManagedStorage, Verbose, "File %ls is added to category %ls", *Filename, *CategoryName);
		}

		return Result;
	}

	bool RemoveFile(const FString& Filename)
	{
		FileSizesLock.WriteLock(); // FWriteScopeLock doesn't have an early unlock :(

		int64 OldSize;
		if (FileSizes.RemoveAndCopyValue(Filename, OldSize))
		{
			FileSizesLock.WriteUnlock();

			UsedQuota -= OldSize;

			UE_LOGF(LogPlatformFileManagedStorage, Verbose, "File %ls is removed from category %ls", *Filename, *CategoryName);

			return true;
		}

		FileSizesLock.WriteUnlock();
		return false;
	}

	void ForEachFile(TFunctionRef<void(const FString&)> Function) const
	{
		FReadScopeLock ScopeLock(FileSizesLock);
		for (const TPair<FString, int64>& Pair : FileSizes)
		{
			Function(Pair.Key);
		}
	}

	const TArray<FString>& GetDirectories() const { return Directories; }

	struct CategoryStat
	{
		FString Print() const
		{
			if (TotalSize < 0)
			{
				return FString::Printf(TEXT("Category %s: %.3f MiB (%" INT64_FMT ") / unlimited used"), *CategoryName, (float)UsedSize / 1024.f / 1024.f, UsedSize);
			}
			else
			{
				return FString::Printf(TEXT("Category %s: %.3f MiB (%" INT64_FMT ") / %.3f MiB used"), *CategoryName, (float)UsedSize / 1024.f / 1024.f, UsedSize, (float)TotalSize / 1024.f / 1024.f);
			}
		}

		FString CategoryName;
		int64 UsedSize = 0;
		int64 TotalSize = -1;
		int64 TotalOptionalSize = 0;
		TMap<FString, int64> FileSizes;
		TArray<FString> Directories;
	};

	// Note this will not be accurate if the category is being modified while this is called but it is low level thread safe
	CategoryStat GetStat() const
	{
		FReadScopeLock ScopeLock(FileSizesLock);
		return CategoryStat{ CategoryName, UsedQuota, StorageQuota, OptionalStorageQuota, FileSizes, Directories };
	}

	FManagedStorageFileLockRegistry& GetLockRegistry() { return FileLockRegistry; }
	const FManagedStorageFileLockRegistry& GetLockRegistry() const { return FileLockRegistry; }

private:
	EPersistentStorageManagerFileSizeFlags TryUpdateQuota(const int64 OldFileSize, const int64 NewFileSize, EPersistentStorageManagerFileSizeFlags Flags)
	{
		check(OldFileSize >= 0);
		check(NewFileSize >= 0);

		if (NewFileSize <= OldFileSize)
		{
			UsedQuota -= (OldFileSize - NewFileSize);
			return EPersistentStorageManagerFileSizeFlags::None;
		}

		if (EnumHasAnyFlags(Flags, EPersistentStorageManagerFileSizeFlags::OnlyUpdateIfLess))
		{
			return EPersistentStorageManagerFileSizeFlags::OnlyUpdateIfLess;
		}

		if (EnumHasAnyFlags(Flags, EPersistentStorageManagerFileSizeFlags::RespectQuota) && StorageQuota >= 0)
		{
			int64 OldUsedQuota = UsedQuota;
			int64 NewUsedQuota;
			do
			{
				NewUsedQuota = OldUsedQuota + (NewFileSize - OldFileSize);
				if (NewUsedQuota > StorageQuota)
				{
					return EPersistentStorageManagerFileSizeFlags::RespectQuota;
				}
			} while (!UsedQuota.compare_exchange_weak(OldUsedQuota, NewUsedQuota));

			return EPersistentStorageManagerFileSizeFlags::None;
		}

		UsedQuota += (NewFileSize - OldFileSize);
		return EPersistentStorageManagerFileSizeFlags::None;
	}

	bool ShouldManageFile(const FString& Filename) const
	{
		for (const FString& Directory : Directories)
		{
			if (ManagedStorageInternal::IsUnderDirectory(Filename, Directory))
			{
				return true;
			}
		}

		return false;
	}

private:
	const FString CategoryName;
	const TArray<FString> Directories;

	const int64 StorageQuota = -1;
	const int64 OptionalStorageQuota = 0;

	std::atomic<int64> UsedQuota{ 0 };

	TMap<FString, int64> FileSizes;
	mutable FRWLock FileSizesLock;

	FManagedStorageFileLockRegistry FileLockRegistry;
};

// NOTE: CORE_API is not used on the whole class because then FCategoryInfo is exported which appears to force the generation
// of copy constructors for FPersistentStorageCategory which causes a compile error because std::atomic can't be copied.
class FPersistentStorageManager
{
public:
	static bool IsReady()
	{
		// FPersistentStorageManager depends on FPaths which depends on the command line being initialized.
		// FPersistentStorageManager depends on GConfig.
		// FPersistentStorageManager can't be constructed until its dependencies are ready
		// FPersistentStorageManager will try and allocate memory during a crash but this could hang during log file flushing
		return GConfig && GConfig->IsReadyForUse() && FCommandLine::IsInitialized() && !GIsCriticalError;
	}

	/** Singleton access **/
	static CORE_API FPersistentStorageManager& Get();

	FPersistentStorageManager();

	void Initialize()
	{
		if (bInitialized)
		{
			return;
		}

		// Check to add files
		AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, []()
		{
			class FInitStorageVisitor : public IPlatformFile::FDirectoryVisitor
			{
			public:
				FInitStorageVisitor() : IPlatformFile::FDirectoryVisitor(EDirectoryVisitorFlags::ThreadSafe) // Go wide with parallel file scan
				{}

				virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
				{
					if (FilenameOrDirectory && !bIsDirectory)
					{
						FPersistentStorageManager& Man = FPersistentStorageManager::Get();
						if (FPersistentManagedFile File = Man.TryManageFile(FilenameOrDirectory))
						{
							FManagedStorageScopeFileLock ScopeFileLock(File);

							// This must be done under the lock because another thread may be modifying or deleting the file while we scan
							FFileStatData StatData = IPlatformFile::GetPlatformPhysical().GetStatData(FilenameOrDirectory);
							if (ensureAlways(StatData.bIsValid))
							{
								if (!ensureAlways(StatData.FileSize >= 0))
								{
									UE_LOGF(LogPlatformFileManagedStorage, Error, "Invalid File Size for %ls!", FilenameOrDirectory);
								}
								
								Man.AddOrUpdateFile(File, StatData.FileSize);
							}
							else
							{
								UE_LOGF(LogPlatformFileManagedStorage, Error, "Invalid Stat Data for %ls!", FilenameOrDirectory);
							}
						}
					}

					return true;
				}
			};

			FInitStorageVisitor Visitor;

			for (const FString& RootDir : FPersistentStorageManager::Get().GetRootDirectories())
			{
				UE_LOGF(LogPlatformFileManagedStorage, Display, "Scan directory %ls", *RootDir);

				IPlatformFile::GetPlatformPhysical().IterateDirectoryRecursively(*RootDir, Visitor);
			}

			UE_LOGF(LogPlatformFileManagedStorage, Display, "Done scanning root directories");
		});

		bInitialized = true;
	}

	FPersistentManagedFile TryManageFile(const FString& Filename)
	{
		FPersistentManagedFile OutFile;
		OutFile.FullFilename = FPaths::ConvertRelativePathToFull(Filename);

		TryManageFileInternal(OutFile);

		return OutFile;
	}

	FPersistentManagedFile TryManageFile(FString&& Filename)
	{
		FPersistentManagedFile OutFile;
		OutFile.FullFilename = FPaths::ConvertRelativePathToFull(MoveTemp(Filename));

		TryManageFileInternal(OutFile);

		return OutFile;
	}

	bool TryManageFile(FPersistentManagedFile& InOutFile)
	{
		check(!InOutFile.IsValid());
		TryManageFileInternal(InOutFile);
		if (InOutFile.IsValid())
		{
			return true;
		}
		InOutFile.SetHasFailedTryToManage();
		return false;
	}

private:
	void TryManageFileInternal(FPersistentManagedFile& OutFile)
	{
		int32 CategoryIndex = 0;
		for (const FPersistentStorageCategory& Category : Categories.GetCategories())
		{
			if (Category.IsInCategory(OutFile.FullFilename))
			{
				OutFile.Category = CategoryIndex;
				return;
			}
			++CategoryIndex;
		}

		bool bIsUnderRootDir = !!Algo::FindByPredicate(RootDirectories.GetRootDirectories(), [&OutFile](const FString& RootDir) { return ManagedStorageInternal::IsUnderDirectory(OutFile.FullFilename, RootDir); });
		if (bIsUnderRootDir)
		{
			OutFile.Category = Categories.GetDefaultCategoryIndex();
		}
	}

public:
	EPersistentStorageManagerFileSizeFlags AddOrUpdateFile(const FPersistentManagedFile& File, const int64 FileSize, 
		EPersistentStorageManagerFileSizeFlags Flags = EPersistentStorageManagerFileSizeFlags::None)
	{
		if (!File)
		{
			return EPersistentStorageManagerFileSizeFlags::None;
		}

		return Categories.GetCategories()[File.Category].AddOrUpdateFile(File.FullFilename, FileSize, Flags);
	}

	bool RemoveFileFromManager(FPersistentManagedFile& File)
	{
		if (!File)
		{
			return false;
		}

		return Categories.GetCategories()[File.Category].RemoveFile(File.FullFilename);
	}

	int64 GetTotalUsedSize() const
	{
		int64 TotalUsedSize = 0LL;
		for(const FPersistentStorageCategory& Category : Categories.GetCategories())
		{
			TotalUsedSize += Category.GetUsedSize();
		}

		return TotalUsedSize;
	}

	/// <summary>
	/// 
	/// </summary>
	/// <param name="Path">path to check for free space</param>
	/// <param name="UsedSpace">amount of used space</param>
	/// <param name="RemainingSpace">amount of remaining free space</param>
	/// <param name="Quota">total storage allocated to the related category</param>
	/// <param name="OptionalQuota">subset of the storage which is optional i.e. will always be smaller then total Quota</param>
	/// <returns>returns if function succeeds</returns>
	bool GetPersistentStorageUsage(FString Path, int64& UsedSpace, int64 &RemainingSpace, int64& Quota, int64* OptionalQuota = nullptr)
	{
		FString FullPath = FPaths::ConvertRelativePathToFull(MoveTemp(Path));

		for (const FPersistentStorageCategory& Category : Categories.GetCategories())
		{
			if (Category.IsInCategory(FullPath))
			{
				UsedSpace = Category.GetUsedSize();
				RemainingSpace = Category.GetAvailableSize();
				Quota = Category.GetCategoryQuota();
				if (OptionalQuota != nullptr)
				{
					*OptionalQuota = Category.GetCategoryOptionalQuota();
				}
				return true;
			}
		}
		return false;
	}

	bool GetPersistentStorageUsageByCategory(const FString& InCategory, int64& UsedSpace, int64& RemainingSpace, int64& Quota, int64* OptionalQuota = nullptr)
	{
		FPersistentStorageCategory* Category = Algo::FindBy(Categories.GetCategories(), InCategory, [](const FPersistentStorageCategory& Cat) { return Cat.GetCategoryName(); });
		if (Category)
		{
			UsedSpace = Category->GetUsedSize();
			RemainingSpace = Category->GetAvailableSize();
			Quota = Category->GetCategoryQuota();
			if (OptionalQuota != nullptr)
			{
				*OptionalQuota = Category->GetCategoryOptionalQuota();
			}
			return true;
		}
		return false;
	}

	/// <summary>
	/// Returns any additional required free space and optional free space
	/// </summary>
	/// <param name="RequiredSpace">Required persistent storage space to run</param>
	/// <param name="OptionalSpace">persistent storage categories marked as optional</param>
	/// <returns></returns>
	bool GetPersistentStorageSize(int64& UsedSpace, int64& RequiredSpace, int64& OptionalSpace) const
	{
		for (const FPersistentStorageCategory& Category : Categories.GetCategories())
		{
			UsedSpace += Category.GetUsedSize();
			RequiredSpace += Category.GetCategoryQuota();
			OptionalSpace += Category.GetCategoryOptionalQuota();
		}
		return true;
	}

	bool IsInitialized() const
	{
		return bInitialized;
	}

	bool IsCategoryForFileFull(const FPersistentManagedFile& File) const
	{
		if (!File)
		{
			return false;
		}

		return Categories.GetCategories()[File.Category].IsCategoryFull();
	}

	TMap<FString, FPersistentStorageCategory::CategoryStat> GenerateCategoryStats() const
	{
		TMap<FString, FPersistentStorageCategory::CategoryStat> CategoryStats;
		CategoryStats.Reserve(Categories.GetCategories().Num());

		for (const FPersistentStorageCategory& Category : Categories.GetCategories())
		{
			CategoryStats.Add(Category.GetCategoryName(), Category.GetStat());
		}

		return CategoryStats;
	}

	TOptional<FPersistentStorageCategory::CategoryStat> GetCategoryStat(const FString& InCategory) const
	{
		TOptional<FPersistentStorageCategory::CategoryStat> Result;

		for (const FPersistentStorageCategory& Category : Categories.GetCategories())
		{
			if (Category.GetCategoryName() == InCategory)
			{
				Result.Emplace(Category.GetStat());
				break;
			}
		}

		return Result;
	}

	TArrayView<const FString> GetRootDirectories() const { return RootDirectories.GetRootDirectories(); }

	// Wrapper for Category Array to prevent resizing after init
	struct FCategoryInfo
	{
	private:
		TArray<FPersistentStorageCategory> Categories;
		int32 DefaultCategory = -1;

	public:
		FCategoryInfo(TArray<FPersistentStorageCategory>&& InCategories, int32 InDefaultCategory);

		TArrayView<FPersistentStorageCategory> GetCategories() { return MakeArrayView(Categories); }
		TArrayView<const FPersistentStorageCategory> GetCategories() const { return MakeArrayView(Categories); }

		FPersistentStorageCategory* GetDefaultCategory() { return DefaultCategory >= 0 ? &Categories[DefaultCategory] : nullptr; }
		const FPersistentStorageCategory* GetDefaultCategory() const { return DefaultCategory >= 0 ? &Categories[DefaultCategory] : nullptr; }

		int32 GetDefaultCategoryIndex() const { return DefaultCategory; }
	};

	const FCategoryInfo& GetCategories() const { return Categories; }

private:
	friend class FManagedStorageScopeFileLock; // For access to Categories

	bool bInitialized;

	// Top level of all managed storage
	// Wrapper to prevent changing or resizing after init
	struct FRootDirInfo
	{
	private:
		TArray<FString> RootDirectories;

	public:
		void Init(TArrayView<const FPersistentStorageCategory> Categories);
		TArrayView<const FString> GetRootDirectories() const { return MakeArrayView(RootDirectories); }
	};
	FRootDirInfo RootDirectories;

	FCategoryInfo Categories;

	static FCategoryInfo InitCategories();
};

// Only write handle 
class FManagedStorageFileWriteHandle : public IFileHandle
{
private:
	bool TryManageFile()
	{
		if (File.IsValid())
		{
			return true;
		}

		if (!File.HasFailedTryToManage() && FPersistentStorageManager::IsReady())
		{
			return FPersistentStorageManager::Get().TryManageFile(File);
		}

		return false;
	}

public:
	FManagedStorageFileWriteHandle(IFileHandle* InFileHandle, FPersistentManagedFile InFile)
		: FileHandle(InFileHandle)
		, File(MoveTemp(InFile))
	{
	}

	virtual ~FManagedStorageFileWriteHandle()
	{
		if (!TryManageFile())
		{
			return;
		}

		FManagedStorageScopeFileLock ScopeFileLock(File);

		FPersistentStorageManager::Get().AddOrUpdateFile(File, FileHandle->Size());
	}

	virtual int64 Tell() override
	{
		return FileHandle->Tell();
	}

	virtual bool Seek(int64 NewPosition) override
	{
		return FileHandle->Seek(NewPosition);
	}

	virtual bool SeekFromEnd(int64 NewPositionRelativeToEnd = 0) override
	{
		return FileHandle->SeekFromEnd(NewPositionRelativeToEnd);
	}

	virtual bool Read(uint8* Destination, int64 BytesToRead) override
	{
		return FileHandle->Read(Destination, BytesToRead);
	}

	virtual bool ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset) override
	{
		return FileHandle->ReadAt(Destination, BytesToRead, Offset);
	}

	virtual bool Write(const uint8* Source, int64 BytesToWrite) override
	{
		if (!TryManageFile())
		{
			return FileHandle->Write(Source, BytesToWrite);
		}

		FPersistentStorageManager& Manager = FPersistentStorageManager::Get();

		FManagedStorageScopeFileLock ScopeFileLock(File);

		int64 NewSize = FMath::Max(FileHandle->Tell() + BytesToWrite, FileHandle->Size());

		EPersistentStorageManagerFileSizeFlags Result = Manager.AddOrUpdateFile(File, NewSize, EPersistentStorageManagerFileSizeFlags::RespectQuota);
		bool bIsFileCategoryFull = Result != EPersistentStorageManagerFileSizeFlags::None;
		if (bIsFileCategoryFull)
		{
			UE_LOGF(LogPlatformFileManagedStorage, Error, "Failed to write to file %ls.  The category of the file has reach quota limit in peristent storage.", *File.FullFilename);
			return false;
		}

		bool bSuccess = FileHandle->Write(Source, BytesToWrite);
		if (!bSuccess)
		{
			int64 FileSize = FileHandle->Size();
			if (ensureAlways(FileSize >= 0))
			{
				verify(Manager.AddOrUpdateFile(File, FileSize) == EPersistentStorageManagerFileSizeFlags::None);
			}
		}

		return bSuccess;
	}

	virtual int64 Size() override
	{
		return FileHandle->Size();
	}

	virtual bool Flush(const bool bFullFlush = false) override
	{
		const bool bOldIsValid = File.IsValid();
		if (!TryManageFile())
		{
			return FileHandle->Flush(bFullFlush);
		}

		FManagedStorageScopeFileLock ScopeFileLock(File);

		const bool bSuccess = FileHandle->Flush(bFullFlush);
		const bool bForceSizeUpdate = !bOldIsValid;
		if (!bSuccess || bForceSizeUpdate)
		{
			int64 FileSize = FileHandle->Size();
			if (ensureAlways(FileSize >= 0))
			{
				verify(FPersistentStorageManager::Get().AddOrUpdateFile(File, FileSize) == EPersistentStorageManagerFileSizeFlags::None);
			}
		}

		return bSuccess;
	}

	virtual bool Truncate(int64 NewSize) override
	{
		if (!TryManageFile())
		{
			return FileHandle->Truncate(NewSize);
		}

		FPersistentStorageManager& Manager = FPersistentStorageManager::Get();

		FManagedStorageScopeFileLock ScopeFileLock(File);

		int64 FileSize = FileHandle->Size();

		if (NewSize <= FileSize)
		{
			bool bSuccess = FileHandle->Truncate(NewSize);
			FileSize = FileHandle->Size();
			verify(Manager.AddOrUpdateFile(File, FileSize) == EPersistentStorageManagerFileSizeFlags::None);
			return bSuccess;
		}

		if (Manager.AddOrUpdateFile(File, NewSize, EPersistentStorageManagerFileSizeFlags::RespectQuota) != EPersistentStorageManagerFileSizeFlags::None)
		{
			UE_LOGF(LogPlatformFileManagedStorage, Error, "Failed to truncate file %ls.  The category of the file has reach quota limit in peristent storage.", *File.FullFilename);
			return false;
		}

		if (!FileHandle->Truncate(NewSize))
		{
			FileSize = FileHandle->Size();
			verify(Manager.AddOrUpdateFile(File, FileSize) == EPersistentStorageManagerFileSizeFlags::None);
			return false;
		}

		return true;
	}

	virtual void ShrinkBuffers() override
	{
		FileHandle->ShrinkBuffers();
	}

private:
	TUniquePtr<IFileHandle>	FileHandle;
	FPersistentManagedFile	File;
};

namespace ManagedStoragePlatformFile
{
	template<typename FuncType>
	struct FDeduceBaseClassHelper;

	template<typename ClassType, typename RetType, typename... ParamTypes>
	struct FDeduceBaseClassHelper<RetType(ClassType::*)(ParamTypes...)>
	{
		using Type = ClassType;
	};

	// Deduces the class the member is defined in. Behavior can be inconsistent if the type brought into the class with a using statement.
	// ex. 
	//		using IPhysicalPlatformFile::OpenWrite;
	// Clang still works as expected, but this a compile error on MSVC. Still this is *good enough* for what is needed here.
	// I think this is probably a bug in MSVC because decltype(&BaseClass::Func) works as expected, even with a using statement.
	template<typename RetType, typename... ParamTypes, typename ClassType>
	constexpr auto MakeDeduceBaseClassHelper(RetType(ClassType::*)(ParamTypes...))
	{
		return FDeduceBaseClassHelper<RetType(ClassType::*)(ParamTypes...)>{};
	}

	template<typename T>
	constexpr bool IsOpenWriteBoolOverriden()
	{
		auto Deduce = MakeDeduceBaseClassHelper<IFileHandle*, const TCHAR*, bool, bool>(&T::OpenWrite);
		constexpr bool bIsOverriden = !std::is_same<typename decltype(Deduce)::Type, IPlatformFile>::value;
		return bIsOverriden;
	}

	template<typename T>
	constexpr bool IsOpenWriteFlagsOverriden()
	{
		auto Deduce = MakeDeduceBaseClassHelper<FFileOpenResult, const TCHAR*, IPlatformFile::EOpenWriteFlags>(&T::OpenWrite);
		constexpr bool bIsOverriden = !std::is_same<typename decltype(Deduce)::Type, IPlatformFile>::value;
		return bIsOverriden;
	}

	// OpenWrite has two overloads - if a derived class only overloads one of them it will 'shadow' the other base overload.
	// The concepts test that the overload is both visible and overriden.
	template<typename T>
	concept CHasOpenWriteBoolOverride = requires(T& Obj, const TCHAR* F, bool bAppend, bool bAllowRead)
	{
		{ Obj.T::OpenWrite(F, bAppend, bAllowRead) } -> UE::CSameAs<IFileHandle*>;
	} && IsOpenWriteBoolOverriden<T>();
	template<typename T>
	concept CHasOpenWriteFlagsOverride = requires(T& Obj, const TCHAR* F, IPlatformFile::EOpenWriteFlags Flags)
	{
		{ Obj.T::OpenWrite(F, Flags) } -> UE::CSameAs<FFileOpenResult>;
	} && IsOpenWriteFlagsOverriden<T>();
}

// NOTE: This is templated rather than a polymorphic wrapper because a lot code expects the physical layer not to be a wrapper.
// It also has the benefit of not needing updating every time a new function is added to IPlatformFile.
template<class BaseClass>
class TManagedStoragePlatformFile : public BaseClass
{
private:
	static bool IsReady()
	{
		// FManagedStoragePlatformFile will just pass through to LowerLevel until FPersistentStorageManager is ready
		return FPersistentStorageManager::IsReady();
	}

public:

	TManagedStoragePlatformFile() : BaseClass()
	{
		static_assert(UE::CDerivedFrom<BaseClass, IPlatformFile>);
	}

	virtual bool DeleteFile(const TCHAR* Filename) override
	{
		if (!IsReady())
		{
			return BaseClass::DeleteFile(Filename);
		}

		FPersistentStorageManager& Manager = FPersistentStorageManager::Get();

		FPersistentManagedFile ManagedFile = Manager.TryManageFile(Filename);

		FManagedStorageScopeFileLock ScopeFileLock(ManagedFile);

		bool bSuccess = BaseClass::DeleteFile(Filename);
		if (bSuccess)
		{
			Manager.RemoveFileFromManager(ManagedFile);
		}

		return bSuccess;
	}

	virtual bool DeleteFiles(const TArrayView<const TCHAR*>& Filenames) override
	{
		constexpr bool bIsOverriden = !std::is_same<
			decltype(&BaseClass::DeleteFiles),
			decltype(&IPlatformFile::DeleteFiles)>::value;

		if constexpr (!bIsOverriden)
		{
			return BaseClass::DeleteFiles(Filenames);
		}
		else
		{
			if (!IsReady())
			{
				return BaseClass::DeleteFiles(Filenames);
			}

			// This should work even if the physical implementation calls ends up calling TManagedStoragePlatformFile::DeleteFile

			FPersistentStorageManager& Manager = FPersistentStorageManager::Get();

			TArray<TPair<FPersistentManagedFile, FManagedStorageScopeFileLock>> FileLocks;
			FileLocks.Reserve(Filenames.Num());
			for (const TCHAR* Filename : Filenames)
			{
				FPersistentManagedFile ManagedFile = Manager.TryManageFile(Filename);
				FileLocks.Emplace(ManagedFile, MoveTemp(ManagedFile));
			}

			bool bSuccess = BaseClass::DeleteFiles(Filenames);

			const int32 Count = Filenames.Num();
			for (int32 i = 0; i < Count; i++)
			{
				if (!BaseClass::FileExists(Filenames[i]))
				{
					UE_LOGF(LogPlatformFileManagedStorage, Display, "Removing deleted file %ls.", *(FileLocks[i].Key.FullFilename));
					Manager.RemoveFileFromManager(FileLocks[i].Key);
				}
				else
				{
					UE_LOGF(LogPlatformFileManagedStorage, Warning, "Not removing deleted file %ls. It still exists on disk.", *(FileLocks[i].Key.FullFilename));
				}
			}

			return bSuccess;
		}
	}

	virtual bool MoveFile(const TCHAR* To, const TCHAR* From) override
	{
		if (!IsReady())
		{
			return BaseClass::MoveFile(To, From);
		}

		FPersistentStorageManager& Manager = FPersistentStorageManager::Get();

		FPersistentManagedFile ManagedTo = Manager.TryManageFile(To);
		FPersistentManagedFile ManagedFrom = Manager.TryManageFile(From);

		FManagedStorageScopeFileLock ScopeFileLockTo(ManagedTo);
		FManagedStorageScopeFileLock ScopeFileLockFrom(ManagedFrom);

		const int64 SizeFrom = this->FileSize(From);
		if (SizeFrom < 0)
		{
			return false;
		}

		EPersistentStorageManagerFileSizeFlags Result = Manager.AddOrUpdateFile(ManagedTo, SizeFrom, EPersistentStorageManagerFileSizeFlags::OnlyUpdateIfLess | EPersistentStorageManagerFileSizeFlags::RespectQuota);
		if (EnumHasAnyFlags(Result, EPersistentStorageManagerFileSizeFlags::RespectQuota))
		{
			UE_LOGF(LogPlatformFileManagedStorage, Error, "Failed to move file to %ls.  The target category of the destination has reach quota limit in peristent storage.", To);
			return false;
		}

		bool bSuccess = BaseClass::MoveFile(To, From);
		if (bSuccess)
		{
			Manager.RemoveFileFromManager(ManagedFrom);
			Manager.AddOrUpdateFile(ManagedTo, SizeFrom);
		}
		else
		{
			// On some implementations MoveFile can operate across volumes, so don't make assumptions about the state of
			// of the file system in the case of failure.

			if (ManagedFrom && !this->FileExists(From))
			{
				Manager.RemoveFileFromManager(ManagedFrom);
			}

			if (ManagedTo)
			{
				if (!this->FileExists(To))
				{
					Manager.RemoveFileFromManager(ManagedTo);
				}
				else
				{
					const int64 SizeTo = this->FileSize(To);
					if (ensureAlways(SizeTo >= 0))
					{
						Manager.AddOrUpdateFile(ManagedTo, SizeTo);
					}
					else
					{
						Manager.RemoveFileFromManager(ManagedTo);
					}
				}
			}
		}

		return bSuccess;
	}

	virtual IFileHandle* OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override
	{
		constexpr bool bOverriden = ManagedStoragePlatformFile::CHasOpenWriteBoolOverride<BaseClass>;
		if constexpr (!bOverriden)
		{
			// Need to use IPlatformFile here. If the BaseClass doesn't override this overload, it will not be visible through the BaseClass because it will be 
			// shadowed by the other version that it does override.
			return IPlatformFile::OpenWrite(Filename, bAppend, bAllowRead);
		}
		else
		{
			static_assert(!ManagedStoragePlatformFile::CHasOpenWriteFlagsOverride<BaseClass>, "Physical Platform files overrides both overloads of OpenWrite!");

			if (!IsReady())
			{
				// Always wrap handle if not ready.  FManagedStorageFileWriteHandle will start managing the file
				// internally when we become ready.
				IFileHandle* InnerHandle = BaseClass::OpenWrite(Filename, bAppend, bAllowRead);
				if (!InnerHandle)
				{
					return nullptr;
				}

				FPersistentManagedFile ManagedFile;
				ManagedFile.FullFilename = FPaths::ConvertRelativePathToFull(Filename);
				return new FManagedStorageFileWriteHandle(InnerHandle, MoveTemp(ManagedFile));
			}

			FPersistentStorageManager& Manager = FPersistentStorageManager::Get();

			FPersistentManagedFile ManagedFile = Manager.TryManageFile(Filename);

			if (Manager.IsCategoryForFileFull(ManagedFile))
			{
				UE_LOGF(LogPlatformFileManagedStorage, Error, "Failed to open file %ls for write.  The category of the file has reach quota limit in peristent storage.", Filename);
				return nullptr;
			}

			FManagedStorageScopeFileLock ScopeFileLock(ManagedFile);

			IFileHandle* InnerHandle = BaseClass::OpenWrite(Filename, bAppend, bAllowRead);
			if (!InnerHandle)
			{
				return nullptr;
			}

			Manager.AddOrUpdateFile(ManagedFile, InnerHandle->Size());
			if (ManagedFile)
			{
				return new FManagedStorageFileWriteHandle(InnerHandle, MoveTemp(ManagedFile));
			}
			else
			{
				return InnerHandle;
			}
		}
	}

	virtual FFileOpenResult OpenWrite(const TCHAR* Filename, IPlatformFile::EOpenWriteFlags Flags) override
	{
		constexpr bool bOverriden = ManagedStoragePlatformFile::CHasOpenWriteFlagsOverride<BaseClass>;
		if constexpr (!bOverriden)
		{
			// Need to use IPlatformFile here. If the BaseClass doesn't override this overload, it will not be visible through the BaseClass because it will be 
			// shadowed by the other version that it does override.
			return IPlatformFile::OpenWrite(Filename, Flags);
		}
		else
		{
			static_assert(!ManagedStoragePlatformFile::CHasOpenWriteBoolOverride<BaseClass>, "Physical Platform files overrides both overloads of OpenWrite!");

			if (!IsReady())
			{
				// Always wrap handle if not ready.  FManagedStorageFileWriteHandle will start managing the file
				// internally when we become ready.
				FFileOpenResult Result = BaseClass::OpenWrite(Filename, Flags);
				if (Result.HasError())
				{
					return Result;
				}

				FPersistentManagedFile ManagedFile;
				ManagedFile.FullFilename = FPaths::ConvertRelativePathToFull(Filename);
				IFileHandle* ManagedHandle = new FManagedStorageFileWriteHandle(Result.GetValue().Release(), MoveTemp(ManagedFile));
				Result = MakeValue(ManagedHandle);
				return Result;
			}

			FPersistentStorageManager& Manager = FPersistentStorageManager::Get();

			FPersistentManagedFile ManagedFile = Manager.TryManageFile(Filename);

			if (Manager.IsCategoryForFileFull(ManagedFile))
			{
				UE_LOGF(LogPlatformFileManagedStorage, Error, "Failed to open file %ls for write. The category of the file has reach quota limit in peristent storage.", Filename);
				return MakeError(FString::Printf(TEXT("Failed to open file %s for write. The category of the file has reach quota limit in peristent storage."), Filename));
			}

			FManagedStorageScopeFileLock ScopeFileLock(ManagedFile);

			FFileOpenResult Result = BaseClass::OpenWrite(Filename, Flags);
			if (Result.HasError())
			{
				return Result;
			}

			Manager.AddOrUpdateFile(ManagedFile, Result.GetValue()->Size());
			if (ManagedFile)
			{
				IFileHandle* ManagedHandle = new FManagedStorageFileWriteHandle(Result.GetValue().Release(), MoveTemp(ManagedFile));
				Result = MakeValue(ManagedHandle);
				return Result;
			}
			else
			{
				return Result;
			}
		}
	}

	virtual FOpenMappedResult OpenMappedEx2(const TCHAR* Filename, IPlatformFile::EOpenMappedFlags OpenOptions = IPlatformFile::EOpenMappedFlags::None, int64 MaximumSize = 0) override
	{
		if (EnumHasAnyFlags(OpenOptions, IPlatformFile::EOpenMappedFlags::AllowWrite))
		{
			UE_LOGF(LogPlatformFileManagedStorage, Error, "TManagedStoragePlatformFile does not support writing files via OpenMappedEx2!");
			ensure(false);
		}

		return BaseClass::OpenMappedEx2(Filename, OpenOptions, MaximumSize);
	}

	virtual bool DeleteDirectoryRecursively(const TCHAR* Directory) override
	{
		constexpr bool bIsOverriden = !std::is_same<
			decltype(&BaseClass::DeleteDirectoryRecursively), 
			decltype(&IPlatformFile::DeleteDirectoryRecursively)>::value;

		if constexpr (!bIsOverriden)
		{
			return BaseClass::DeleteDirectoryRecursively(Directory);
		}
		else
		{
			FPersistentStorageManager& Manager = FPersistentStorageManager::Get();

			// Access Categories directly. We will take the registry lock for all affected categories.
			// We need to consider other threads adding the directory while we are trying to remove it.
			const FPersistentStorageManager::FCategoryInfo& CategoryInfo = Manager.GetCategories();

			FString FullDirectory = FPaths::ConvertRelativePathToFull(Directory);

			TArray<FScopeLock> LockRegistryLocks;
			TArray<FPersistentManagedFile> Files;

			for (int32 CategoryIndex = 0; CategoryIndex < CategoryInfo.GetCategories().Num(); ++CategoryIndex)
			{
				const FPersistentStorageCategory& Category = CategoryInfo.GetCategories()[CategoryIndex];

				bool bSearchCategory = Category.IsInCategory(FullDirectory);
				if (!bSearchCategory)
				{
					// Also need to check if the directory being removed contains category dirs
					for (const FString& CatDir : Category.GetDirectories())
					{
						if (ManagedStorageInternal::IsUnderDirectory(CatDir, FullDirectory))
						{
							bSearchCategory = true;
							break;
						}
					}
				}

				if (!bSearchCategory)
				{
					continue;
				}

				// This doesn't "lock" a catgory. It keeps new file locks from being added or removed for files in 
				// the category while we transact.
				LockRegistryLocks.Emplace(Category.GetLockRegistry().GetRegistryLock());
				
				auto AddFilesInDirectory = [&FullDirectory, &Files, CategoryIndex](const FString& FilePath)
				{
					if (ManagedStorageInternal::IsUnderDirectory(FilePath, FullDirectory))
					{
						FPersistentManagedFile& ManagedFile = Files.Emplace_GetRef();
						ManagedFile.FullFilename = FilePath;
						ManagedFile.Category = CategoryIndex;
					}
				};

				Category.ForEachFile(AddFilesInDirectory);

				// Also add check for any in-flight files that may have taken out a lock but may not be added to the category yet.
				// We need to lock those files so we don't race with adding them.
				Category.GetLockRegistry().ForEachFile(UE::DeferLock, AddFilesInDirectory);
			}

			// FCriticalSection is reentrant so we can do while holding the category registry lock
			TArray<FManagedStorageScopeFileLock> FilesLocks;
			FilesLocks.Reserve(Files.Num());
			for (const FPersistentManagedFile& File : Files)
			{
				FilesLocks.Emplace(File);
			}

			const bool bSuccess = BaseClass::DeleteDirectoryRecursively(Directory);
			
			// Ok for other operations on affected categories at this point. We still have locks on all possibly removed files.
			LockRegistryLocks.Reset();

			for (FPersistentManagedFile& File : Files)
			{
				if (!BaseClass::FileExists(*File.FullFilename))
				{
					UE_LOGF(LogPlatformFileManagedStorage, Display, "Removing deleted file %ls.", *File.FullFilename);
					Manager.RemoveFileFromManager(File);
				}
				else
				{
					UE_LOGF(LogPlatformFileManagedStorage, Warning, "Not removing deleted file %ls. It still exists on disk.", *File.FullFilename);
				}
			}

			return bSuccess;
		}
	}

	virtual bool CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags = EPlatformFileRead::None, EPlatformFileWrite WriteFlags = EPlatformFileWrite::None) override
	{
		if (!IsReady())
		{
			return BaseClass::CopyFile(To, From, ReadFlags, WriteFlags);
		}

		FPersistentStorageManager& Manager = FPersistentStorageManager::Get();

		FPersistentManagedFile ManagedTo = Manager.TryManageFile(To);
		FPersistentManagedFile ManagedFrom = Manager.TryManageFile(From);

		FManagedStorageScopeFileLock ScopeFileLockTo(ManagedTo);
		FManagedStorageScopeFileLock ScopeFileLockFrom(ManagedFrom);

		const int64 SizeFrom = this->FileSize(From);
		if (SizeFrom < 0)
		{
			return false;
		}

		EPersistentStorageManagerFileSizeFlags Result = Manager.AddOrUpdateFile(ManagedTo, SizeFrom, EPersistentStorageManagerFileSizeFlags::OnlyUpdateIfLess | EPersistentStorageManagerFileSizeFlags::RespectQuota);
		if (EnumHasAnyFlags(Result, EPersistentStorageManagerFileSizeFlags::RespectQuota))
		{
			UE_LOGF(LogPlatformFileManagedStorage, Error, "Failed to copy file to %ls. The category of the destination has reach quota limit in peristent storage.", To);
			return false;
		}

		bool bSuccess = BaseClass::CopyFile(To, From, ReadFlags, WriteFlags);
		if (bSuccess)
		{
			Manager.AddOrUpdateFile(ManagedTo, SizeFrom);
		}
		else if(ManagedTo)
		{
			if (!this->FileExists(To))
			{
				Manager.RemoveFileFromManager(ManagedTo);
			}
			else
			{
				const int64 SizeTo = this->FileSize(To);
				if (ensureAlways(SizeTo >= 0))
				{
					Manager.AddOrUpdateFile(ManagedTo, SizeTo);
				}
				else
				{
					Manager.RemoveFileFromManager(ManagedTo);
				}
			}
		}

		return bSuccess;
	}

	virtual bool CopyDirectoryTree(const TCHAR* DestinationDirectory, const TCHAR* Source, bool bOverwriteAllExisting) override
	{
		static_assert(std::is_same<decltype(&BaseClass::CopyDirectoryTree), decltype(&IPlatformFile::CopyDirectoryTree)>::value, "TManagedStoragePlatformFile does not support overriding CopyDirectoryTree!");
		return BaseClass::CopyDirectoryTree(DestinationDirectory, Source, bOverwriteAllExisting);
	}
};
