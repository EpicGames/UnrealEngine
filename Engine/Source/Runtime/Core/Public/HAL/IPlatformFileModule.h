// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CoreFwd.h"
#include "Modules/ModuleInterface.h"

/**
 * Platform File Module Interface
 */
class IPlatformFileModule
	: public IModuleInterface
{
public:

	/**
	 * Creates a platform file instance.
	 *
	 * @return Platform file instance.
	 */
	virtual IPlatformFile* GetPlatformFile() = 0;
};
