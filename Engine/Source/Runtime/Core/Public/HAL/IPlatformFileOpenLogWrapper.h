// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncFileHandle.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProperties.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/DateTime.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"

class FPlatformFileOpenLog;
class IAsyncReadFileHandle;
class IMappedFileHandle;

#if !UE_BUILD_SHIPPING

#define FILE_OPEN_ORDER_LABEL(Format, ...) \
	FPlatformFileOpenLog::AddLabel( Format, ##__VA_ARGS__ );

class FLoggingAsyncReadFileHandle final : public IAsyncReadFileHandle
{
	FPlatformFileOpenLog* Owner;
	FString Filename;
	IAsyncReadFileHandle* ActualRequest;
public:
	FLoggingAsyncReadFileHandle(FPlatformFileOpenLog* InOwner, const TCHAR* InFilename, IAsyncReadFileHandle* InActualRequest)
		: Owner(InOwner)
		, Filename(InFilename)
		, ActualRequest(InActualRequest)
	{
	}
	~FLoggingAsyncReadFileHandle()
	{
		delete ActualRequest;
	}
	virtual IAsyncReadRequest* SizeRequest(FAsyncFileCallBack* CompleteCallback = nullptr) override
	{
		return ActualRequest->SizeRequest(CompleteCallback);
	}
	virtual IAsyncReadRequest* ReadRequest(int64 Offset, int64 BytesToRead, EAsyncIOPriorityAndFlags PriorityAndFlags = AIOP_Normal, FAsyncFileCallBack* CompleteCallback = nullptr, uint8* UserSuppliedMemory = nullptr) override;
	
	virtual bool UsesCache() override
	{
		return ActualRequest->UsesCache();
	}
};


class FPlatformFileOpenLog : public IPlatformFile
{
protected:

	IPlatformFile*			LowerLevel;
	FCriticalSection		CriticalSection;
	int64					OpenOrder;
	TMap<FString, int64>	FilenameAccessMap;
	TArray<IFileHandle*>	LogOutput;
	bool					bLogDuplicates;

public:

	FPlatformFileOpenLog()
		: LowerLevel(nullptr)
		, OpenOrder(0)
		, bLogDuplicates(false)
	{
	}

	virtual ~FPlatformFileOpenLog() = default;

	//~ For visibility of overloads we don't override
	using IPlatformFile::IterateDirectory;
	using IPlatformFile::IterateDirectoryRecursively;
	using IPlatformFile::IterateDirectoryStat;
	using IPlatformFile::IterateDirectoryStatRecursively;

	virtual bool ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const override
	{
		bool bResult = FParse::Param(CmdLine, TEXT("FileOpenLog")) || FParse::Param(CmdLine, TEXT("FilePackageOpenLog"));
		return bResult;
	}

	virtual bool Initialize(IPlatformFile* Inner, const TCHAR* CommandLineParam) override;

	virtual IPlatformFile* GetLowerLevel() override
	{
		return LowerLevel;
	}
	virtual void SetLowerLevel(IPlatformFile* NewLowerLevel) override
	{
		LowerLevel = NewLowerLevel;
	}
	static const TCHAR* GetTypeName()
	{
		return TEXT("FileOpenLog");
	}
	virtual const TCHAR* GetName() const override
	{
		return GetTypeName();
	}
	virtual bool		FileExists(const TCHAR* Filename) override
	{
		return LowerLevel->FileExists(Filename);
	}
	virtual int64		FileSize(const TCHAR* Filename) override
	{
		return LowerLevel->FileSize(Filename);
	}
	virtual bool		DeleteFile(const TCHAR* Filename) override
	{
		return LowerLevel->DeleteFile(Filename);
	}
	virtual bool		IsReadOnly(const TCHAR* Filename) override
	{
		return LowerLevel->IsReadOnly(Filename);
	}
	virtual bool		MoveFile(const TCHAR* To, const TCHAR* From) override
	{
		return LowerLevel->MoveFile(To, From);
	}
	virtual bool		SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override
	{
		return LowerLevel->SetReadOnly(Filename, bNewReadOnlyValue);
	}
	virtual FDateTime	GetTimeStamp(const TCHAR* Filename) override
	{
		return LowerLevel->GetTimeStamp(Filename);
	}
	virtual void		SetTimeStamp(const TCHAR* Filename, FDateTime DateTime) override
	{
		LowerLevel->SetTimeStamp(Filename, DateTime);
	}
	virtual FDateTime	GetAccessTimeStamp(const TCHAR* Filename) override
	{
		return LowerLevel->GetAccessTimeStamp(Filename);
	}
	virtual FString	GetFilenameOnDisk(const TCHAR* Filename) override
	{
		return LowerLevel->GetFilenameOnDisk(Filename);
	}
	virtual IFileHandle*	OpenRead(const TCHAR* Filename, bool bAllowWrite) override
	{
		IFileHandle* Result = LowerLevel->OpenRead(Filename, bAllowWrite);
		if (Result)
		{
			AddToOpenLog(Filename);
		}
		return Result;
	}
	virtual IFileHandle*	OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override
	{
		return LowerLevel->OpenWrite(Filename, bAppend, bAllowRead);
	}
	virtual bool		DirectoryExists(const TCHAR* Directory) override
	{
		return LowerLevel->DirectoryExists(Directory);
	}
	virtual bool		CreateDirectory(const TCHAR* Directory) override
	{
		return LowerLevel->CreateDirectory(Directory);
	}
	virtual bool		DeleteDirectory(const TCHAR* Directory) override
	{
		return LowerLevel->DeleteDirectory(Directory);
	}
	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override
	{
		return LowerLevel->GetStatData(FilenameOrDirectory);
	}
	virtual bool		IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override
	{
		return LowerLevel->IterateDirectory( Directory, Visitor );
	}
	virtual bool		IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override
	{
		return LowerLevel->IterateDirectoryRecursively( Directory, Visitor );
	}
	virtual bool		IterateDirectoryStat(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) override
	{
		return LowerLevel->IterateDirectoryStat( Directory, Visitor );
	}
	virtual bool		IterateDirectoryStatRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) override
	{
		return LowerLevel->IterateDirectoryStatRecursively( Directory, Visitor );
	}
	virtual bool		DeleteDirectoryRecursively(const TCHAR* Directory) override
	{
		return LowerLevel->DeleteDirectoryRecursively( Directory );
	}
	virtual bool		CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags = EPlatformFileRead::None, EPlatformFileWrite WriteFlags = EPlatformFileWrite::None) override
	{
		return LowerLevel->CopyFile( To, From, ReadFlags, WriteFlags);
	}
	virtual bool		CreateDirectoryTree(const TCHAR* Directory) override
	{
		return LowerLevel->CreateDirectoryTree(Directory);
	}
	virtual bool		CopyDirectoryTree(const TCHAR* DestinationDirectory, const TCHAR* Source, bool bOverwriteAllExisting) override
	{
		return LowerLevel->CopyDirectoryTree(DestinationDirectory, Source, bOverwriteAllExisting);
	}
	virtual FString		ConvertToAbsolutePathForExternalAppForRead( const TCHAR* Filename ) override
	{
		return LowerLevel->ConvertToAbsolutePathForExternalAppForRead(Filename);
	}
	virtual FString		ConvertToAbsolutePathForExternalAppForWrite( const TCHAR* Filename ) override
	{
		return LowerLevel->ConvertToAbsolutePathForExternalAppForWrite(Filename);
	}
	virtual bool		SendMessageToServer(const TCHAR* Message, IFileServerMessageHandler* Handler) override
	{
		return LowerLevel->SendMessageToServer(Message, Handler);
	}
	virtual bool		FileJournalIsAvailable(const TCHAR* VolumeOrPath = nullptr,
		ELogVerbosity::Type* OutErrorLevel = nullptr, FString* OutError = nullptr) override
	{
		return LowerLevel->FileJournalIsAvailable(VolumeOrPath, OutErrorLevel, OutError);
	}
	virtual bool		FileJournalIterateDirectory(const TCHAR* Directory,
		FDirectoryJournalVisitorFunc Visitor, FString* OutError = nullptr) override
	{
		return LowerLevel->FileJournalIterateDirectory(Directory, Visitor, OutError);
	}
	virtual EFileJournalResult FileJournalGetLatestEntry(const TCHAR* VolumeName, FFileJournalId& OutJournalId,
		FFileJournalEntryHandle& OutEntryHandle, FString* OutError = nullptr) override
	{
		return LowerLevel->FileJournalGetLatestEntry(VolumeName, OutJournalId, OutEntryHandle, OutError);
	}
	virtual EFileJournalResult FileJournalReadModified(const TCHAR* VolumeName,
		const FFileJournalId& JournalIdOfStartingEntry, const FFileJournalEntryHandle& StartingJournalEntry,
		TMap<FFileJournalFileHandle, FString>& KnownDirectories, TSet<FString>& OutModifiedDirectories,
		FFileJournalEntryHandle& OutNextJournalEntry, FString* OutError = nullptr) override
	{
		return LowerLevel->FileJournalReadModified(VolumeName, JournalIdOfStartingEntry, StartingJournalEntry,
			KnownDirectories, OutModifiedDirectories, OutNextJournalEntry, OutError);
	}
	virtual FFileJournalData FileJournalGetFileData(const TCHAR* FilenameOrDirectory,
		FString* OutError = nullptr) override
	{
		return LowerLevel->FileJournalGetFileData(FilenameOrDirectory, OutError);
	}
	virtual FString		FileJournalGetVolumeName(FStringView InPath) override
	{
		return LowerLevel->FileJournalGetVolumeName(InPath);
	}
	virtual uint64		FileJournalGetMaximumSize(const TCHAR* VolumeOrPath = nullptr,
		ELogVerbosity::Type* OutErrorLevel = nullptr, FString* OutError = nullptr) const override
	{
		return LowerLevel->FileJournalGetMaximumSize(VolumeOrPath, OutErrorLevel, OutError);
	}
	virtual IAsyncReadFileHandle* OpenAsyncRead(const TCHAR* Filename, bool bAllowWrite = false) override
	{
		// we must not record the "open" here...what matters is when we start reading the file!
		return new FLoggingAsyncReadFileHandle(this, Filename, LowerLevel->OpenAsyncRead(Filename, bAllowWrite));
	}
	virtual FOpenMappedResult OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions = EOpenReadFlags::None, int64 MaximumSize = 0) override;
	virtual FOpenMappedResult OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions = EOpenMappedFlags::None, int64 MaximumSize = 0) override;

	void AddToOpenLog(const TCHAR* Filename)
	{
		CriticalSection.Lock();
		if (bLogDuplicates)
		{
			FString Text = FString::Printf(TEXT("\"%s\"\n"), Filename);
			for (auto File = LogOutput.CreateIterator(); File; ++File)
			{
				(*File)->Write((uint8*)StringCast<ANSICHAR>(*Text).Get(), Text.Len());
			}
		}
		else if (FilenameAccessMap.Find(Filename) == nullptr)
		{
			FilenameAccessMap.Emplace(Filename, ++OpenOrder);
			FString Text = FString::Printf(TEXT("\"%s\" %llu\n"), Filename, OpenOrder);
			for (auto File = LogOutput.CreateIterator(); File; ++File)
			{
				(*File)->Write((uint8*)StringCast<ANSICHAR>(*Text).Get(), Text.Len());
			}
		}
		CriticalSection.Unlock();
	}

	void AddLabelToOpenLog(const TCHAR* LabelStr)
	{
		CriticalSection.Lock();
		for (auto File = LogOutput.CreateIterator(); File; ++File)
		{
			FString Text = FString::Printf(TEXT("# %s\n"), LabelStr);
			(*File)->Write((uint8*)StringCast<ANSICHAR>(*Text).Get(), Text.Len());
		}
		CriticalSection.Unlock();
	}


	void AddPackageToOpenLog(const TCHAR* Filename)
	{
		// TODO: deprecate/remove this function?
		AddToOpenLog(Filename);
	}

	template <typename FmtType, typename... Types>
	static void AddLabel(const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FCsvProfiler::AddLabel");
		AddLabelInternal((const TCHAR*)Fmt, Args...);
	}

private:
	static CORE_API void VARARGS AddLabelInternal(const TCHAR* Fmt, ...);

};
#else // !UE_BUILD_SHIPPING

#define FILE_OPEN_ORDER_LABEL(Format, ...)

#endif

