// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/LowLevelMemTracker.h"
#include "Stats/Stats.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER && STATS
/**
 * LLM Stat implementation macros; these macros are used publicly in LLM_SCOPED_SINGLE_PLATFORM_STAT_TAG and privately in LLM implementation
 */
	#define DECLARE_LLM_MEMORY_STAT(CounterName,StatId,GroupId) \
		DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_int64, EStatFlags::None, FPlatformMemory::MCR_PhysicalLLM); \
		static DEFINE_STAT(StatId)
	#define DECLARE_LLM_MEMORY_STAT_EXTERN(CounterName,StatId,GroupId, API) \
		DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_int64, EStatFlags::None, FPlatformMemory::MCR_PhysicalLLM); \
		extern API DEFINE_STAT(StatId);

	DECLARE_STATS_GROUP(TEXT("LLM FULL"), STATGROUP_LLMFULL, STATCAT_Advanced);
	DECLARE_STATS_GROUP(TEXT("LLM Platform"), STATGROUP_LLMPlatform, STATCAT_Advanced);
	DECLARE_STATS_GROUP(TEXT("LLM Summary"), STATGROUP_LLM, STATCAT_Advanced);
	DECLARE_STATS_GROUP(TEXT("LLM Overhead"), STATGROUP_LLMOverhead, STATCAT_Advanced);
	DECLARE_STATS_GROUP(TEXT("LLM Assets"), STATGROUP_LLMAssets, STATCAT_Advanced);
	DECLARE_STATS_GROUP(TEXT("LLM UObjects"), STATGROUP_LLMUObjects, STATCAT_Advanced);
	DECLARE_STATS_GROUP(TEXT("LLM CodeOrContent"), STATGROUP_LLMCodeOrContent, STATCAT_Advanced);

	/*
	* LLM Summary stats referenced by ELLMTagNames
	*/
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Engine"), STAT_EngineSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Project"), STAT_ProjectSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Networking"), STAT_NetworkingSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Audio"), STAT_AudioSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Total"), STAT_TrackedTotalSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Meshes"), STAT_MeshesSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Physics"), STAT_PhysicsSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("PhysX"), STAT_PhysXSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Chaos"), STAT_ChaosSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("UObject"), STAT_UObjectSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Animation"), STAT_AnimationSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("StaticMesh"), STAT_StaticMeshSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Materials"), STAT_MaterialsSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Particles"), STAT_ParticlesSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Niagara"), STAT_NiagaraSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("UI"), STAT_UISummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Navigation"), STAT_NavigationSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Textures"), STAT_TexturesSummaryLLM, STATGROUP_LLM, CORE_API);
	DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("MediaStreaming"), STAT_MediaStreamingSummaryLLM, STATGROUP_LLM, CORE_API);

#else
	#define DECLARE_LLM_MEMORY_STAT(CounterName,StatId,GroupId)
	#define DECLARE_LLM_MEMORY_STAT_EXTERN(CounterName,StatId,GroupId, API)

#endif //ENABLE_LOW_LEVEL_MEM_TRACKER && STATS

/**
 * LLM require-Stats scope macros. These are noops if LLM or STATS is defined out, and when defined in, are inactive unless
 * toggled on by LLM command line options.
 */
#if ENABLE_LOW_LEVEL_MEM_TRACKER && STATS
#define LLM_SCOPED_TAG_WITH_STAT(Stat, Tracker) FLLMScope SCOPE_NAME(GET_STATFNAME(Stat), true /* bIsStatTag */, ELLMTagSet::None, Tracker);
#define LLM_SCOPED_TAG_WITH_STAT_IN_SET(Stat, Set, Tracker) FLLMScope SCOPE_NAME(GET_STATFNAME(Stat), true /* bIsStatTag */, Set, Tracker);
#define LLM_SCOPED_TAG_WITH_STAT_NAME(StatName, Tracker) FLLMScope SCOPE_NAME(StatName, true /* bIsStatTag */, ELLMTagSet::None, Tracker);
#define LLM_SCOPED_TAG_WITH_STAT_NAME_IN_SET(StatName, Set, Tracker) FLLMScope SCOPE_NAME(StatName, true /* bIsStatTag */, Set, Tracker);
#define LLM_SCOPED_SINGLE_PLATFORM_STAT_TAG(Stat) DECLARE_LLM_MEMORY_STAT(TEXT(#Stat), Stat, STATGROUP_LLMPlatform); LLM_SCOPED_TAG_WITH_STAT(Stat, ELLMTracker::Platform);
#define LLM_SCOPED_SINGLE_PLATFORM_STAT_TAG_IN_SET(Stat, Set) DECLARE_LLM_MEMORY_STAT(TEXT(#Stat), Stat, STATGROUP_LLMPlatform); LLM_SCOPED_TAG_WITH_STAT_IN_SET(Stat, Set, ELLMTracker::Platform);
#define LLM_SCOPED_SINGLE_STAT_TAG(Stat) DECLARE_LLM_MEMORY_STAT(TEXT(#Stat), Stat, STATGROUP_LLMFULL); LLM_SCOPED_TAG_WITH_STAT(Stat, ELLMTracker::Default);
#define LLM_SCOPED_SINGLE_STAT_TAG_IN_SET(Stat, Set) DECLARE_LLM_MEMORY_STAT(TEXT(#Stat), Stat, STATGROUP_LLMFULL); LLM_SCOPED_TAG_WITH_STAT_IN_SET(Stat, Set, ELLMTracker::Default);
#define LLM_SCOPED_PAUSE_TRACKING_WITH_STAT_AND_AMOUNT(Stat, Amount, Tracker) FLLMPauseScope SCOPE_NAME(GET_STATFNAME(Stat), true /* bIsStatTag */, Amount, Tracker, ELLMAllocType::None);
#else
#define LLM_SCOPED_TAG_WITH_STAT(...)
#define LLM_SCOPED_TAG_WITH_STAT_IN_SET(...)
#define LLM_SCOPED_TAG_WITH_STAT_NAME(...)
#define LLM_SCOPED_TAG_WITH_STAT_NAME_IN_SET(...)
#define LLM_SCOPED_SINGLE_PLATFORM_STAT_TAG(...)
#define LLM_SCOPED_SINGLE_PLATFORM_STAT_TAG_IN_SET(...)
#define LLM_SCOPED_SINGLE_STAT_TAG(...)
#define LLM_SCOPED_SINGLE_STAT_TAG_IN_SET(...)
#define LLM_SCOPED_PAUSE_TRACKING_WITH_STAT_AND_AMOUNT(...)
#endif

/**
 * LLM optionally-use-Stats scope macros. These are active even if STATS is defined out; they still create LLM tag
 * scopes. If STATS is defined in, they also create a STAT matching the tag (if their tagset is active in LLM). They
 * are noops if LLM is defined out. When defined in, they are inactive unless toggled on by LLM command line options.
 */
#if ENABLE_LOW_LEVEL_MEM_TRACKER

#define LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Object, Set) LLM_SCOPE_DYNAMIC(FName(*Object->GetPathName()), \
					ELLMTracker::Default, Set, FLLMDynamicTagConstructorStatString(Object->GetPathName()))
#define LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(ObjectPath, Set) LLM_SCOPE_DYNAMIC(ObjectPath, \
					ELLMTracker::Default, Set, FLLMDynamicTagConstructorStatString((ObjectPath).ToString()))

// Macro to update threads after each asset when tracking assets
// Currently this is unused, but we may use it for optimizations later
//#define LLM_PUSH_STATS_FOR_ASSET_TAGS() if (FLowLevelMemTracker::Get().IsTagSetActive(ELLMTagSet::Assets)) {}
#define LLM_PUSH_STATS_FOR_ASSET_TAGS()

#else // ENABLE_LOW_LEVEL_MEM_TRACKER

#define LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(...)
#define LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(...)
#define LLM_PUSH_STATS_FOR_ASSET_TAGS()

#endif // else !ENABLE_LOW_LEVEL_MEM_TRACKER

#define LLM_SCOPED_TAG_WITH_OBJECT_IN_SET(Object, Set) UE_DEPRECATED_MACRO(5.3, "Use LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH instead") LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Object, Set);


