// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "LowLevelMemTrackerDefines.h" // LLM_ENABLED_IN_CONFIG
#include "ProfilingDebugging/TagTrace.h"
#include "AutoRTFM.h"

#ifndef PLATFORM_SUPPORTS_LLM
	#define PLATFORM_SUPPORTS_LLM 1
#endif

#define LLM_ENABLED_ON_PLATFORM UE_DEPRECATED_MACRO(5.7, "Use PLATFORM_SUPPORTS_LLM instead") (PLATFORM_SUPPORTS_LLM)

#ifdef ENABLE_LOW_LEVEL_MEM_TRACKER
	#error ENABLE_LOW_LEVEL_MEM_TRACKER is now a derived define that should not be defined separately. Define LLM_ENABLED_IN_CONFIG (build environment only) or PLATFORM_SUPPORTS_LLM (build environment or c++ header) instead.
#endif
#define ENABLE_LOW_LEVEL_MEM_TRACKER (LLM_ENABLED_IN_CONFIG && PLATFORM_SUPPORTS_LLM)

// Deprecated defines no longer used
#ifndef LLM_ALLOW_ASSETS_TAGS
#define LLM_ALLOW_ASSETS_TAGS UE_DEPRECATED_MACRO(5.8, "LLM_ALLOW_ASSETS_TAGS is deprecated and should always be considered 1 when ENABLE_LOW_LEVEL_MEM_TRACKER is 1.") ENABLE_LOW_LEVEL_MEM_TRACKER
#endif
#ifndef LLM_ALLOW_UOBJECTCLASSES_TAGS
#define LLM_ALLOW_UOBJECTCLASSES_TAGS UE_DEPRECATED_MACRO(5.8, "LLM_ALLOW_UOBJECTCLASSES_TAGS is deprecated and should always be considered 1 when ENABLE_LOW_LEVEL_MEM_TRACKER is 1.") ENABLE_LOW_LEVEL_MEM_TRACKER
#endif
#ifndef LLM_ALLOW_STATS
#define LLM_ALLOW_STATS UE_DEPRECATED_MACRO(5.8, "LLM_ALLOW_STATS is deprecated and should always be considered 1 when ENABLE_LOW_LEVEL_MEM_TRACKER is 1.") ENABLE_LOW_LEVEL_MEM_TRACKER
#endif
#define LLM_ENABLED_STAT_TAGS UE_DEPRECATED_MACRO(5.8, "LLM_ENABLED_STAT_TAGS is deprecated and should always be considered 1 when ENABLE_LOW_LEVEL_MEM_TRACKER is 1.") ENABLE_LOW_LEVEL_MEM_TRACKER

// Define LLM macros to refer to the implementation class if LLM is compiled in, otherwise define them to NOOP.
#if ENABLE_LOW_LEVEL_MEM_TRACKER

// Notes about LLM CommandLine options
// LLMTagSets: Assets: the feature can be toggled on at runtime with commandline -llmtagsets=assets.
// Toggling the feature on causes a huge number of stat ids to be created and has a high cputime cost.
// When toggled on, AssetTags report the asset that is in scope for each allocation
// LLM Assets can be viewed in game using 'Stat LLMAssets'.

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FTagTrace;

#if DO_CHECK

namespace UE::LLMPrivate
{

bool HandleAssert(bool bLog, const TCHAR* Format, ...);

// LLMEnsure's use of this generates a bool per callsite by passing a lambda which uniquely instantiates the template.
template <typename Type>
bool TrueOnFirstCallOnly(const Type&)
{
	static bool bValue = true;
	bool Result = bValue;
	bValue = false;
	return Result;
}

} // UE::LLMPrivate

#if !USING_CODE_ANALYSIS
	#define LLMTrueOnFirstCallOnly			UE::LLMPrivate::TrueOnFirstCallOnly([]{})
#else
	#define LLMTrueOnFirstCallOnly			false
#endif

#define LLMCheckMessage(expr)          TEXT("LLM check failed: %s [File:%s] [Line: %d]\r\n"),             TEXT(#expr), TEXT(__FILE__), __LINE__
#define LLMCheckfMessage(expr, format) TEXT("LLM check failed: %s [File:%s] [Line: %d]\r\n") format TEXT("\r\n"),      TEXT(#expr), TEXT(__FILE__), __LINE__
#define LLMEnsureMessage(expr)         TEXT("LLM ensure failed: %s [File:%s] [Line: %d]\r\n"),            TEXT(#expr), TEXT(__FILE__), __LINE__

#define LLMCheck(expr)					do { if (UNLIKELY(!(expr))) { UE::LLMPrivate::HandleAssert(true, LLMCheckMessage(expr));                         FPlatformMisc::RaiseException(1); } } while(false)
#define LLMCheckf(expr,format,...)		do { if (UNLIKELY(!(expr))) { UE::LLMPrivate::HandleAssert(true, LLMCheckfMessage(expr, format), ##__VA_ARGS__); FPlatformMisc::RaiseException(1); } } while(false)
#define LLMEnsure(expr) (LIKELY(!!(expr)) || UE::LLMPrivate::HandleAssert(LLMTrueOnFirstCallOnly, LLMEnsureMessage(expr)))

#else

#define LLMCheck(expr)
#define LLMCheckf(expr,...)
#define LLMEnsure(expr)	(!!(expr))

#endif

#define LLM_TAG_TYPE uint8

// estimate the maximum amount of memory LLM will need to run on a game with around 4 million allocations.
// Make sure that you have debug memory enabled on consoles (on screen warning will show if you don't)
// (currently only used on PS4 to stop it reserving a large chunk up front. This will go away with the new memory system)
#define LLM_MEMORY_OVERHEAD (600LL*1024*1024)

/**
 * LLM Trackers
 */
enum class ELLMTracker : uint8
{
	/**
	 * "Platform" indicates process heap space that is allocated directly from the kernel.
	 * This value tracks pages committed by the operating system (VirtualAlloc, mmap).
	 */
	Platform,

	/**
	 * "Default" means memory that is both physically committed and logically allocated.
	 * This value tracks memory actively in use by Unreal's allocators (FMemory::Malloc/Free, NewObject<T>, etc).
	 */
	Default,

	Max,
};

/*
 * optional tags that need to be enabled with -llmtagsets=x,y,z on the commandline
 */
enum class ELLMTagSet : uint8
{
	None,
	/** Create tag for each Asset, and assign allocations for any UObject in a package to the package's asset. */
	Assets,
	/**
	 * Create tag for each UObject class, and assign allocations for any UObject in a package to the class of the
	 * package's asset.
	 */
	AssetClasses,
	/** Create tag for each UObject class, and assign allocations for any UObject to the class of the UObject. */
	UObjectClasses,
	/**
	 * Create two tags - Code or Content. Assign allocations for any UObject to the Content tag; assign allocations
	 * shared by multiple objects or unrelated to UObjects to the Code tag.
	 */
	CodeOrContent,
	
	Max, // note: update functions that handle ELLMTagSets if you add any new tagsets
};

namespace UE::LLM
{

/*
 * Size parameter flags used when requesting the size of the tracked tag data.
 */
enum class ESizeParams : uint8
{
	Default = 0,
	ReportCurrent = 0,
	ReportPeak = 1,
	RelativeToSnapshot = 2
};

ENUM_CLASS_FLAGS(ESizeParams);

/*
 * Returns the preferred directory path for LLM profiling output files (i.e.: Saved/Profiling/LLM/).
 */
FString GetLLMProfilingDir();

} // UE::LLM

// Do not add to these macros. Please use the LLM_DECLARE_TAG family of macros below to create new tags.
#define LLM_ENUM_GENERIC_TAGS(macro) \
	macro(Untagged,								"Untagged",						NAME_None,													NAME_None,										-1)\
	macro(Paused,								"Paused",						NAME_None,													NAME_None,										-1)\
	macro(Total,								"Total",						GET_STATFNAME(STAT_TotalLLM),								GET_STATFNAME(STAT_TrackedTotalSummaryLLM),		-1)\
	macro(Untracked,							"Untracked",					GET_STATFNAME(STAT_UntrackedLLM),							GET_STATFNAME(STAT_TrackedTotalSummaryLLM),		-1)\
	macro(PlatformTotal,						"Total",						GET_STATFNAME(STAT_PlatformTotalLLM),						NAME_None,										-1)\
	macro(TrackedTotal,							"TrackedTotal",					GET_STATFNAME(STAT_TrackedTotalLLM),						GET_STATFNAME(STAT_TrackedTotalSummaryLLM),		-1)\
	macro(UntaggedTotal,						"Untagged",						GET_STATFNAME(STAT_UntaggedTotalLLM),						NAME_None,										-1)\
	macro(WorkingSetSize,						"WorkingSetSize",				GET_STATFNAME(STAT_WorkingSetSizeLLM),						GET_STATFNAME(STAT_TrackedTotalSummaryLLM),		-1)\
	macro(PagefileUsed,							"PagefileUsed",					GET_STATFNAME(STAT_PagefileUsedLLM),						GET_STATFNAME(STAT_TrackedTotalSummaryLLM),		-1)\
	macro(PlatformTrackedTotal,					"TrackedTotal",					GET_STATFNAME(STAT_PlatformTrackedTotalLLM),				NAME_None,										-1)\
	macro(PlatformUntaggedTotal,				"Untagged",						GET_STATFNAME(STAT_PlatformUntaggedTotalLLM),				NAME_None,										-1)\
	macro(PlatformUntracked,					"Untracked",					GET_STATFNAME(STAT_PlatformUntrackedLLM),					NAME_None,										-1)\
	macro(PlatformOverhead,						"LLMOverhead",					GET_STATFNAME(STAT_PlatformOverheadLLM),					NAME_None,										-1)\
	macro(PlatformOSAvailable,					"OSAvailable",					GET_STATFNAME(STAT_PlatformOSAvailableLLM),					NAME_None,										-1)\
	/*FMalloc is a special tag that is reserved for the Platform Tracker only. It's used with ELLMAllocType::FMalloc to calculate ELLMTag::FMallocUnused. */								   \
	macro(FMalloc,								"FMalloc",						GET_STATFNAME(STAT_FMallocLLM),								NAME_None,										-1)\
	macro(FMallocUnused,						"FMallocUnused",				GET_STATFNAME(STAT_FMallocUnusedLLM),						GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(RHIUnused,							"RHIUnused",					GET_STATFNAME(STAT_RHIUnusedLLM),							GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(ThreadStack,							"ThreadStack",					GET_STATFNAME(STAT_ThreadStackLLM),							GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(ThreadStackPlatform,					"ThreadStack",					GET_STATFNAME(STAT_ThreadStackPlatformLLM),					NAME_None,										-1)\
	macro(ProgramSizePlatform,					"ProgramSize",					GET_STATFNAME(STAT_ProgramSizePlatformLLM),					NAME_None,										-1)\
	macro(ProgramSize,							"ProgramSize",					GET_STATFNAME(STAT_ProgramSizeLLM),							GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(BackupOOMMemoryPoolPlatform,			"OOMBackupPool",				GET_STATFNAME(STAT_OOMBackupPoolPlatformLLM),				NAME_None,										-1)\
	macro(BackupOOMMemoryPool,					"OOMBackupPool",				GET_STATFNAME(STAT_OOMBackupPoolLLM),						GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(GenericPlatformMallocCrash,			"GenericPlatformMallocCrash",	GET_STATFNAME(STAT_GenericPlatformMallocCrashLLM),			GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(GenericPlatformMallocCrashPlatform,	"GenericPlatformMallocCrash",	GET_STATFNAME(STAT_GenericPlatformMallocCrashPlatformLLM),	GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	/* Any low-level memory that is not tracked in any other category. */ \
	macro(EngineMisc,							"EngineMisc",					GET_STATFNAME(STAT_EngineMiscLLM),							GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	/* Any task kicked off from the task graph that doesn't have its own category. Should be fairly low. */ \
	macro(TaskGraphTasksMisc,					"TaskGraphMiscTasks",			GET_STATFNAME(STAT_TaskGraphTasksMiscLLM),					GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(LinearAllocator,						"LinearAllocator",				GET_STATFNAME(STAT_LinearAllocatorLLM),						NAME_None,										-1)\
	macro(Audio,								"Audio",						GET_STATFNAME(STAT_AudioLLM),								GET_STATFNAME(STAT_AudioSummaryLLM),			-1)\
	macro(AudioMisc,							"AudioMisc",					GET_STATFNAME(STAT_AudioMiscLLM),							GET_STATFNAME(STAT_AudioSummaryLLM),			ELLMTag::Audio)\
	macro(AudioSoundWaves,						"AudioSoundWaves",				GET_STATFNAME(STAT_AudioSoundWavesLLM),						GET_STATFNAME(STAT_AudioSummaryLLM),			ELLMTag::Audio)\
	macro(AudioSoundWaveProxies,				"AudioSoundWaveProxies",		GET_STATFNAME(STAT_AudioSoundWaveProxiesLLM),				GET_STATFNAME(STAT_AudioSummaryLLM),			ELLMTag::Audio)\
	macro(AudioMixer,							"AudioMixer",					GET_STATFNAME(STAT_AudioMixerLLM),							GET_STATFNAME(STAT_AudioSummaryLLM),			ELLMTag::Audio)\
	macro(AudioMixerPlugins,					"AudioMixerPlugins",			GET_STATFNAME(STAT_AudioMixerPluginsLLM),					GET_STATFNAME(STAT_AudioSummaryLLM),			ELLMTag::Audio)\
	macro(AudioPrecache,						"AudioPrecache",				GET_STATFNAME(STAT_AudioPrecacheLLM),						GET_STATFNAME(STAT_AudioSummaryLLM),			ELLMTag::Audio)\
	macro(AudioDecompress,						"AudioDecompress",				GET_STATFNAME(STAT_AudioDecompressLLM),						GET_STATFNAME(STAT_AudioSummaryLLM),			ELLMTag::Audio)\
	macro(AudioRealtimePrecache,				"AudioRealtimePrecache",		GET_STATFNAME(STAT_AudioRealtimePrecacheLLM),				GET_STATFNAME(STAT_AudioSummaryLLM),			ELLMTag::Audio)\
	macro(AudioFullDecompress,					"AudioFullDecompress",			GET_STATFNAME(STAT_AudioFullDecompressLLM),					GET_STATFNAME(STAT_AudioSummaryLLM),			ELLMTag::Audio)\
	macro(AudioStreamCache,						"AudioStreamCache",				GET_STATFNAME(STAT_AudioStreamCacheLLM),					GET_STATFNAME(STAT_AudioSummaryLLM),			ELLMTag::Audio)\
	macro(AudioStreamCacheCompressedData,		"AudioStreamCacheCompressedData",GET_STATFNAME(STAT_AudioStreamCacheCompressedDataLLM),		GET_STATFNAME(STAT_AudioSummaryLLM),			ELLMTag::Audio)\
	macro(AudioSynthesis,						"AudioSynthesis",				GET_STATFNAME(STAT_AudioSynthesisLLM),						GET_STATFNAME(STAT_AudioSummaryLLM),			ELLMTag::Audio)\
	macro(RealTimeCommunications,				"RealTimeCommunications",		GET_STATFNAME(STAT_RealTimeCommunicationsLLM),				NAME_None,										-1)\
	macro(FName,								"FName",						GET_STATFNAME(STAT_FNameLLM),								GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(FNameCompressed,						"FNameCompressed",				GET_STATFNAME(STAT_FNameCompressedLLM),						NAME_None,										ELLMTag::FName)\
	macro(Networking,							"Networking",					GET_STATFNAME(STAT_NetworkingLLM),							GET_STATFNAME(STAT_NetworkingSummaryLLM),		-1)\
	macro(Meshes,								"Meshes",						GET_STATFNAME(STAT_MeshesLLM),								GET_STATFNAME(STAT_MeshesSummaryLLM),			-1)\
	macro(Stats,								"Stats",						GET_STATFNAME(STAT_StatsLLM),								GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(Shaders,								"Shaders",						GET_STATFNAME(STAT_ShadersLLM),								GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(PSO,									"PSO",							GET_STATFNAME(STAT_PSOLLM),									GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(Textures,								"Textures",						GET_STATFNAME(STAT_TexturesLLM),							GET_STATFNAME(STAT_TexturesSummaryLLM),			-1)\
	macro(TextureMetaData,						"TextureMetaData",				GET_STATFNAME(STAT_TextureMetaDataLLM),						GET_STATFNAME(STAT_TexturesSummaryLLM),			-1)\
	macro(VirtualTextureSystem,					"VirtualTextureSystem",			GET_STATFNAME(STAT_VirtualTextureSystemLLM),				GET_STATFNAME(STAT_TexturesSummaryLLM),			-1)\
	macro(RenderTargets,						"RenderTargets",				GET_STATFNAME(STAT_RenderTargetsLLM),						GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(SceneRender,							"SceneRender",					GET_STATFNAME(STAT_SceneRenderLLM),							GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(RHIMisc,								"RHIMisc",						GET_STATFNAME(STAT_RHIMiscLLM),								GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(AsyncLoading,							"AsyncLoading",					GET_STATFNAME(STAT_AsyncLoadingLLM),						GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
    /* UObject is a catch-all for all Engine and game memory that is not tracked in any other category. */ \
	/* it includes any class inherited from UObject and anything that is serialized by that class including properties. */ \
	/* Note that this stat doesn't include Mesh or Animation data which are tracked separately. */ \
	/* It is correlated to the number of Objects placed in the Level. */ \
	macro(UObject,								"UObject",						GET_STATFNAME(STAT_UObjectLLM),								GET_STATFNAME(STAT_UObjectSummaryLLM),			-1)\
	macro(Animation,							"Animation",					GET_STATFNAME(STAT_AnimationLLM),							GET_STATFNAME(STAT_AnimationSummaryLLM),		-1)\
	/* This is the UStaticMesh class and related properties, and does not include the actual mesh data. */ \
	macro(StaticMesh,							"StaticMesh",					GET_STATFNAME(STAT_StaticMeshLLM),							GET_STATFNAME(STAT_StaticMeshSummaryLLM),		ELLMTag::Meshes)\
	macro(Materials,							"Materials",					GET_STATFNAME(STAT_MaterialsLLM),							GET_STATFNAME(STAT_MaterialsSummaryLLM),		-1)\
	macro(Particles,							"Particles",					GET_STATFNAME(STAT_ParticlesLLM),							GET_STATFNAME(STAT_ParticlesSummaryLLM),		-1)\
	macro(Niagara,								"Niagara",						GET_STATFNAME(STAT_NiagaraLLM),								GET_STATFNAME(STAT_NiagaraSummaryLLM),			-1)\
	macro(GPUSort,								"GPUSort",						GET_STATFNAME(STAT_GPUSortLLM),								GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(GC,									"GC",							GET_STATFNAME(STAT_GCLLM),									GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(UI,									"UI",							GET_STATFNAME(STAT_UILLM),									GET_STATFNAME(STAT_UISummaryLLM),				-1)\
	macro(NavigationRecast,						"NavigationRecast",				GET_STATFNAME(STAT_NavigationRecastLLM),					GET_STATFNAME(STAT_NavigationSummaryLLM),		-1)\
	macro(Physics,								"Physics",						GET_STATFNAME(STAT_PhysicsLLM),								GET_STATFNAME(STAT_PhysicsSummaryLLM),			-1)\
	macro(PhysX,								"PhysX",						GET_STATFNAME(STAT_PhysXLLM),								GET_STATFNAME(STAT_PhysXSummaryLLM),			ELLMTag::Physics)\
	macro(PhysXGeometry,						"PhysXGeometry",				GET_STATFNAME(STAT_PhysXGeometryLLM),						GET_STATFNAME(STAT_PhysXSummaryLLM),			ELLMTag::Physics)\
	macro(PhysXTrimesh,							"PhysXTrimesh",					GET_STATFNAME(STAT_PhysXTrimeshLLM),						GET_STATFNAME(STAT_PhysXSummaryLLM),			ELLMTag::Physics)\
	macro(PhysXConvex,							"PhysXConvex",					GET_STATFNAME(STAT_PhysXConvexLLM),							GET_STATFNAME(STAT_PhysXSummaryLLM),			ELLMTag::Physics)\
	macro(PhysXAllocator,						"PhysXAllocator",				GET_STATFNAME(STAT_PhysXAllocatorLLM),						GET_STATFNAME(STAT_PhysXSummaryLLM),			ELLMTag::Physics)\
	macro(PhysXLandscape,						"PhysXLandscape",				GET_STATFNAME(STAT_PhysXLandscapeLLM),						GET_STATFNAME(STAT_PhysXSummaryLLM),			ELLMTag::Physics)\
	macro(Chaos,								"Chaos",						GET_STATFNAME(STAT_ChaosLLM),								GET_STATFNAME(STAT_ChaosSummaryLLM),			ELLMTag::Physics)\
	macro(ChaosGeometry,						"ChaosGeometry",				GET_STATFNAME(STAT_ChaosGeometryLLM),						GET_STATFNAME(STAT_ChaosSummaryLLM),			ELLMTag::Physics)\
	macro(ChaosAcceleration,					"ChaosAcceleration",			GET_STATFNAME(STAT_ChaosAccelerationLLM),					GET_STATFNAME(STAT_ChaosSummaryLLM),			ELLMTag::Physics)\
	macro(ChaosParticles,						"ChaosParticles",				GET_STATFNAME(STAT_ChaosParticlesLLM),						GET_STATFNAME(STAT_ChaosSummaryLLM),			ELLMTag::Physics)\
	macro(ChaosLandscape,						"ChaosLandscape",				GET_STATFNAME(STAT_ChaosLandscapeLLM),						GET_STATFNAME(STAT_ChaosSummaryLLM),			ELLMTag::Physics)\
	macro(ChaosTrimesh,							"ChaosTrimesh",					GET_STATFNAME(STAT_ChaosTrimeshLLM),						GET_STATFNAME(STAT_ChaosSummaryLLM),			ELLMTag::Physics)\
	macro(ChaosConvex,							"ChaosConvex",					GET_STATFNAME(STAT_ChaosConvexLLM),							GET_STATFNAME(STAT_ChaosSummaryLLM),			ELLMTag::Physics)\
	macro(ChaosScene,							"ChaosScene",					GET_STATFNAME(STAT_ChaosSceneLLM),							GET_STATFNAME(STAT_ChaosSummaryLLM),			ELLMTag::Physics)\
	macro(ChaosUpdate,							"ChaosUpdate",					GET_STATFNAME(STAT_ChaosUpdateLLM),							GET_STATFNAME(STAT_ChaosSummaryLLM),			ELLMTag::Physics)\
	macro(ChaosActor,							"ChaosActor",					GET_STATFNAME(STAT_ChaosActorLLM),							GET_STATFNAME(STAT_ChaosSummaryLLM),			ELLMTag::Physics)\
	macro(ChaosBody,							"ChaosBody",					GET_STATFNAME(STAT_ChaosBodyLLM),							GET_STATFNAME(STAT_ChaosSummaryLLM),			ELLMTag::Physics)\
	macro(ChaosConstraint,						"ChaosConstraint",				GET_STATFNAME(STAT_ChaosConstraintLLM),						GET_STATFNAME(STAT_ChaosSummaryLLM),			ELLMTag::Physics)\
	macro(ChaosMaterial,						"ChaosMaterial",				GET_STATFNAME(STAT_ChaosMaterialLLM),						GET_STATFNAME(STAT_ChaosSummaryLLM),			ELLMTag::Physics)\
	macro(EnginePreInitMemory,					"EnginePreInit",				GET_STATFNAME(STAT_EnginePreInitLLM),						GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(EngineInitMemory,						"EngineInit",					GET_STATFNAME(STAT_EngineInitLLM),							GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(RenderingThreadMemory,				"RenderingThread",				GET_STATFNAME(STAT_RenderingThreadLLM),						GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(LoadMapMisc,							"LoadMapMisc",					GET_STATFNAME(STAT_LoadMapMiscLLM),							GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(StreamingManager,						"StreamingManager",				GET_STATFNAME(STAT_StreamingManagerLLM),					GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(GraphicsPlatform,						"Graphics",						GET_STATFNAME(STAT_GraphicsPlatformLLM),					NAME_None,										-1)\
	macro(FileSystem,							"FileSystem",					GET_STATFNAME(STAT_FileSystemLLM),							GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(Localization,							"Localization",					GET_STATFNAME(STAT_LocalizationLLM),						GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(AssetRegistry,						"AssetRegistry",				GET_STATFNAME(STAT_AssetRegistryLLM),						GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(ConfigSystem,							"ConfigSystem",					GET_STATFNAME(STAT_ConfigSystemLLM),						GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(InitUObject,							"InitUObject",					GET_STATFNAME(STAT_InitUObjectLLM),							GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(VideoRecording,						"VideoRecording",				GET_STATFNAME(STAT_VideoRecordingLLM),						GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(Replays,								"Replays",						GET_STATFNAME(STAT_ReplaysLLM),								GET_STATFNAME(STAT_NetworkingSummaryLLM),		ELLMTag::Networking)\
	macro(MaterialInstance,						"MaterialInstance",				GET_STATFNAME(STAT_MaterialInstanceLLM),					GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(SkeletalMesh,							"SkeletalMesh",					GET_STATFNAME(STAT_SkeletalMeshLLM),						GET_STATFNAME(STAT_EngineSummaryLLM),			ELLMTag::Meshes)\
	macro(InstancedMesh,						"InstancedMesh",				GET_STATFNAME(STAT_InstancedMeshLLM),						GET_STATFNAME(STAT_EngineSummaryLLM),			ELLMTag::Meshes)\
	macro(Landscape,							"Landscape",					GET_STATFNAME(STAT_LandscapeLLM),							GET_STATFNAME(STAT_EngineSummaryLLM),			ELLMTag::Meshes)\
	macro(CsvProfiler,							"CsvProfiler",					GET_STATFNAME(STAT_CsvProfilerLLM),							GET_STATFNAME(STAT_EngineSummaryLLM),			-1)\
	macro(MediaStreaming,						"MediaStreaming",				GET_STATFNAME(STAT_MediaStreamingLLM),						GET_STATFNAME(STAT_MediaStreamingSummaryLLM),	-1)\
	macro(ElectraPlayer,						"ElectraPlayer",				GET_STATFNAME(STAT_ElectraPlayerLLM),						GET_STATFNAME(STAT_MediaStreamingSummaryLLM),	ELLMTag::MediaStreaming)\
	macro(WMFPlayer,							"WMFPlayer",					GET_STATFNAME(STAT_WMFPlayerLLM),							GET_STATFNAME(STAT_MediaStreamingSummaryLLM),	ELLMTag::MediaStreaming)\
	macro(PlatformMMIO,							"MMIO",							GET_STATFNAME(STAT_PlatformMMIOLLM),						NAME_None,										-1)\
	macro(PlatformVM,							"Virtual Memory",				GET_STATFNAME(STAT_PlatformVMLLM),							NAME_None,										-1)\
	macro(CustomName,							"CustomName",					GET_STATFNAME(STAT_CustomName),								NAME_None,										-1)\

/*
 * Enum values to be passed in to LLM_SCOPE() macro
 */
enum class ELLMTag : LLM_TAG_TYPE
{
#define LLM_ENUM(Enum,Str,Stat,Group,Parent) Enum,
	LLM_ENUM_GENERIC_TAGS(LLM_ENUM)
#undef LLM_ENUM

	GenericTagCount,

	//------------------------------
	// Platform tags
	PlatformTagStart = 111,
	PlatformTagEnd = 149,

	//------------------------------
	// Project tags
	ProjectTagStart = 150,
	ProjectTagEnd = 255,

	// anything above this value is treated as an FName for a stat section
};
static_assert( ELLMTag::GenericTagCount <= ELLMTag::PlatformTagStart,
	"too many LLM tags defined -- Instead of adding a new tag and updating the limits, please use the LLM_DECLARE_TAG macros below"); 

constexpr inline uint32 LLM_TAG_COUNT = 256;
constexpr inline uint32 LLM_CUSTOM_TAG_START = (int32)ELLMTag::PlatformTagStart;
constexpr inline uint32 LLM_CUSTOM_TAG_END = (int32)ELLMTag::ProjectTagEnd;
constexpr inline uint32 LLM_CUSTOM_TAG_COUNT = LLM_CUSTOM_TAG_END + 1 - LLM_CUSTOM_TAG_START;

/**
 * Passed in to OnLowLevelAlloc to specify the type of allocation. Used to track FMalloc total
 * and pausing for a specific allocation type.
 */
enum class ELLMAllocType
{
	None = 0,
	FMalloc,
	System,
	RHI,
	Count
};

extern const ANSICHAR* LLMGetTagNameANSI(ELLMTag Tag);
extern const TCHAR* LLMGetTagName(ELLMTag Tag);
extern CORE_API const FName LLMGetUntaggedTagName(ELLMTagSet TagSet);
/*
 * LLM utility macros
 */
#define LLM(x) x
#define LLM_IF_ENABLED(x) if (FLowLevelMemTracker::IsEnabled()) { x; }
#define LLM_IS_ENABLED() FLowLevelMemTracker::IsEnabled()
#define SCOPE_NAME UE_JOIN(LLMScope,__LINE__)

///////////////////////////////////////////////////////////////////////////////////////
// These are the main macros to use externally when tracking memory
///////////////////////////////////////////////////////////////////////////////////////

/**
 * LLM scope macros
 */
#define LLM_SCOPE(Tag)												FLLMScope SCOPE_NAME(Tag, false /* bIsStatTag */, ELLMTagSet::None, ELLMTracker::Default);\
																	UE_MEMSCOPE(Tag) 
#define LLM_SCOPE_DYNAMIC(UniqueName, Tracker, TagSet, Constructor) \
	FLLMScopeDynamic SCOPE_NAME(Tracker, TagSet); \
	UE_MEMSCOPE_UNINITIALIZED(__LINE__); \
	do \
	{ \
		if (SCOPE_NAME.IsEnabled()) \
		{ \
			FName UniqueNameEvaluated(UniqueName); \
			if (!UniqueNameEvaluated.IsNone()) \
			{ \
				if (SCOPE_NAME.TryFindTag(UniqueNameEvaluated)) \
				{ \
					SCOPE_NAME.Activate(); \
				} \
				else \
				{ \
					SCOPE_NAME.TryAddTagAndActivate(UniqueNameEvaluated, Constructor); \
				} \
				constexpr int DefaultTracker = (int)ELLMTracker::Default; \
				if (TagSet == ELLMTagSet::None && (int)Tracker == DefaultTracker) \
				{ \
					UE_MEMSCOPE_ACTIVATE(__LINE__, UniqueNameEvaluated); \
				} \
			} \
		} \
	} while (false) /* do/while is added to require a semicolon */

#define LLM_TAGSET_SCOPE(Tag, TagSet)								FLLMScope SCOPE_NAME(Tag, false /* bIsStaTag */, TagSet, ELLMTracker::Default);
#define LLM_SCOPE_BYNAME(Tag) 										static FName UE_JOIN(LLMScope_Name,__LINE__)(Tag);\
																	FLLMScope SCOPE_NAME(UE_JOIN(LLMScope_Name,__LINE__), false /* bIsStatTag */, ELLMTagSet::None, ELLMTracker::Default);\
																	UE_MEMSCOPE(UE_JOIN(LLMScope_Name,__LINE__));
#define LLM_SCOPE_BYTAG(TagDeclName)								FLLMScope SCOPE_NAME(UE_JOIN(LLMTagDeclaration_, TagDeclName).GetUniqueName(), false /* bIsStatTag */, ELLMTagSet::None, ELLMTracker::Default);\
																	UE_MEMSCOPE(UE_JOIN(LLMTagDeclaration_,TagDeclName).GetUniqueName());
#define LLM_SCOPE_RENDER_RESOURCE(Tag)								static const FString UE_JOIN(LLMScope_NamePrefix,__LINE__)(TEXT("RenderResources.")); \
																	FName UE_JOIN(LLMScope_Name,__LINE__)(UE_JOIN(LLMScope_NamePrefix, __LINE__) + (Tag ? Tag : TEXT("Unknown"))); \
																	FLLMScope SCOPE_NAME(UE_JOIN(LLMScope_Name,__LINE__), false /* bIsStatTag */, ELLMTagSet::Assets, ELLMTracker::Default, false /* bOverride */);
#define LLM_PLATFORM_SCOPE(Tag)										FLLMScope SCOPE_NAME(Tag, false /* bIsStatTag */, ELLMTagSet::None, ELLMTracker::Platform);
#define LLM_PLATFORM_SCOPE_BYNAME(Tag) 								static FName UE_JOIN(LLMScope_Name,__LINE__)(Tag);\
																	FLLMScope SCOPE_NAME(UE_JOIN(LLMScope_Name,__LINE__), false /* bIsStatTag */, ELLMTagSet::None, ELLMTracker::Platform);
#define LLM_PLATFORM_SCOPE_BYTAG(TagDeclName)						FLLMScope SCOPE_NAME(UE_JOIN(LLMTagDeclaration_, TagDeclName).GetUniqueName(), false /* bIsStatTag */, ELLMTagSet::None, ELLMTracker::Platform);
#define LLM_SCOPE_CLEAR()											FLLMClearScope SCOPE_NAME(ELLMTagSet::None, ELLMTracker::Default);\
																	UE_MEMSCOPE(0)
#define LLM_TAGSET_SCOPE_CLEAR(TagSet)								FLLMClearScope SCOPE_NAME(TagSet, ELLMTracker::Default);

 /**
 * LLM Pause scope macros
 */
#define LLM_SCOPED_PAUSE_TRACKING(AllocType) 						FLLMPauseScope SCOPE_NAME(ELLMTag::Untagged, false /* bIsStatTag */, 0, ELLMTracker::Max, AllocType);
#define LLM_SCOPED_PAUSE_TRACKING_FOR_TRACKER(Tracker, AllocType) 	FLLMPauseScope SCOPE_NAME(ELLMTag::Untagged, false /* bIsStatTag */, 0, Tracker, AllocType);
#define LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT(Tag, Amount, Tracker, AllocType) FLLMPauseScope SCOPE_NAME(Tag, false /* bIsStatTag */, Amount, Tracker, AllocType);
#define LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(TagDeclName, Amount, Tracker, AllocType) FLLMPauseScope SCOPE_NAME(UE_JOIN(LLMTagDeclaration_, TagDeclName).GetUniqueName(), false /* bIsStatTag */, Amount, Tracker, AllocType);

/**
 * LLM realloc scope macros. Used when reallocating a pointer and you wish to retain the tagging from the source pointer
 */
#define LLM_REALLOC_SCOPE(Ptr)										FLLMScopeFromPtr SCOPE_NAME(Ptr, ELLMTracker::Default);
#define LLM_REALLOC_PLATFORM_SCOPE(Ptr)								FLLMScopeFromPtr SCOPE_NAME(Ptr, ELLMTracker::Platform);

/**
 * LLM tag dumping, to help with identifying mis-tagged items. Probably don't want to check in with these in use!
 */
#define LLM_DUMP_TAG()  FLowLevelMemTracker::Get().DumpTag(ELLMTracker::Default,__FILE__,__LINE__)
#define LLM_DUMP_PLATFORM_TAG()  FLowLevelMemTracker::Get().DumpTag(ELLMTracker::Platform,__FILE__,__LINE__)

/**
 * Define a tag which can be used in LLM_SCOPE_BYTAG or referenced by name in other LLM_SCOPEs.
 * @param UniqueNameWithUnderscores - Token useable as c++ variable name.
 *        Modified version of the name of the tag. Used for looking up by name,
 *        must be unique across all tags passed to LLM_DEFINE_TAG, LLM_SCOPE, or ELLMTag.
 *        The modification: the usual separator / for parents must be replaced with _ in LLM_DEFINE_TAGs.
 * @param DisplayName - (Optional) - FName
*         The name to display when tracing the tag; joined with "/" to the name of its
 *        parent if it has a parent, or NAME_None to use the UniqueName.
 * @param ParentTagName - (Optional) - FName
 *        The unique name of the parent tag, or NAME_None if it has no parent.
 * @param StatName - (Optional) - FName
 *        The name of the stat to populate with this tag's amount when publishing LLM data each
 *        frame, or NAME_None if no stat should be populated.
 * @param SummaryStatName - (Optional) - FName
 *        The name of the stat group to add on this tag's amount when publishing LLM
 *        data each frame, or NAME_None if no stat group should be added to.
 * @param TagSet - (Optional) - ELLMTagSet
 *        The TagSet the FName is assigned to. Default is ELLMTagSet::None.
 */
#define LLM_DEFINE_TAG(UniqueNameWithUnderscores, ...) FLLMTagDeclaration UE_JOIN(LLMTagDeclaration_, UniqueNameWithUnderscores)(TEXT(#UniqueNameWithUnderscores), ##__VA_ARGS__)

/**
 * Statically define a tag which can be used in LLM_SCOPE_BYTAG or referenced by name in other LLM_SCOPEs.
 * @param UniqueNameWithUnderscores - Modified version of the name of the tag. Used for looking up by name,
 *        must be unique across all tags passed to LLM_DEFINE_TAG, LLM_SCOPE, or ELLMTag.
 *        The modification: the usual separator / for parents must be replaced with _ in LLM_DEFINE_TAGs.
 * @param DisplayName - (Optional) - The name to display when tracing the tag; joined with "/" to the name of its
 *        parent if it has a parent, or NAME_None to use the UniqueName.
 * @param ParentTagName - (Optional) - The unique name of the parent tag, or NAME_None if it has no parent.
 * @param StatName - (Optional) - The name of the stat to populate with this tag's amount when publishing LLM data each
 *        frame, or NAME_None if no stat should be populated.
 * @param SummaryStatName - (Optional) - The name of the stat group to add on this tag's amount when publishing LLM
 *        data each frame, or NAME_None if no stat group should be added to.
 */
#define LLM_DEFINE_STATIC_TAG(UniqueNameWithUnderscores, ...) static FLLMTagDeclaration UE_JOIN(LLMTagDeclaration_, UniqueNameWithUnderscores)(TEXT(#UniqueNameWithUnderscores), ##__VA_ARGS__)

 /**
  * Declare a tag defined by LLM_DEFINE_TAG. It is used in LLM_SCOPE_BYTAG or referenced by name in other LLM_SCOPEs.
  * @param UniqueName - The name of the tag for looking up by name, must be unique across all tags passed to
  *        LLM_DEFINE_TAG, LLM_SCOPE, or ELLMTag.
  * @param ModuleName - (Optional, only in LLM_DECLARE_TAG_API) - The MODULENAME_API symbol to use to declare module
  *        linkage, aka ENGINE_API. If omitted, no module linkage will be used and the tag will create link errors if
  *        used from another module.
  */
#define LLM_DECLARE_TAG(UniqueNameWithUnderscores) extern FLLMTagDeclaration UE_JOIN(LLMTagDeclaration_, UniqueNameWithUnderscores)
#define LLM_DECLARE_TAG_API(UniqueNameWithUnderscores, ModuleAPI) extern ModuleAPI FLLMTagDeclaration UE_JOIN(LLMTagDeclaration_, UniqueNameWithUnderscores)

/** Get the unique Name of a Tag. It can be passed to functions such as OnLowLevelAlloc that take a Tag UniqueName. */
#define LLM_TAG_NAME(UniqueNameWithUnderscores) (UE_JOIN(LLMTagDeclaration_, UniqueNameWithUnderscores).GetUniqueName())

/**
 * The BootStrap versions of LLM_DEFINE_TAG, LLM_SCOPE_BYTAG, and LLM_DECLARE_TAG support use in scopes during global
 * c++ constructors, before Main. These tags are slightly more expensive (even when LLM is disabled) in all
 * configurations other than shipping, because they use a function static rather than a global variable.
 */
#define LLM_DEFINE_BOOTSTRAP_TAG(UniqueNameWithUnderscores, ...) \
	FLLMTagDeclaration& UE_JOIN(GetLLMTagDeclaration_, UniqueNameWithUnderscores)() \
	{ \
		static FLLMTagDeclaration UE_JOIN(LLMTagDeclaration_, UniqueNameWithUnderscores)(TEXT(#UniqueNameWithUnderscores), ##__VA_ARGS__); \
		return UE_JOIN(LLMTagDeclaration_, UniqueNameWithUnderscores); \
	}
#define LLM_SCOPE_BY_BOOTSTRAP_TAG(TagDeclName) FLLMScope SCOPE_NAME(UE_JOIN(GetLLMTagDeclaration_, TagDeclName)().GetUniqueName(), false /* bIsStatTag */, ELLMTagSet::None, ELLMTracker::Default);\
												UE_MEMSCOPE(UE_JOIN(GetLLMTagDeclaration_,TagDeclName)().GetUniqueName());
#define LLM_DECLARE_BOOTSTRAP_TAG(UniqueNameWithUnderscores) extern FLLMTagDeclaration& UE_JOIN(GetLLMTagDeclaration_, UniqueNameWithUnderscores)();
#define LLM_DECLARE_BOOTSTRAP_TAG_API(UniqueNameWithUnderscores, ModuleAPI) extern ModuleAPI FLLMTagDeclaration& UE_JOIN(GetLLMTagDeclaration_, UniqueNameWithUnderscores)();

typedef void*(*LLMAllocFunction)(size_t);
typedef void(*LLMFreeFunction)(void*, size_t);

class FLLMTagDeclaration;

struct FLLMTagSetAllocationFilter;

namespace UE::LLMPrivate
{

class FLLMGlobals;
class FTagData;
struct FEnableStateScopeLock;

enum class EEnabled : uint8
{
	NotYetKnown = 0,
	Disabled,
	Enabled,
};

typedef void (*FLLMInitialisedCallback)(UPTRINT UserData);
typedef void (*FTagCreationCallback)(const UE::LLMPrivate::FTagData* TagData, UPTRINT UserData);

/**
 * Callbacks that can occur during LLM Initialisation. These happen before main and are therefore
 * dangerous to use, so they are private.
 */
struct FPrivateCallbacks
{
private:
	CORE_API static void AddInitialisedCallback(FLLMInitialisedCallback Callback, UPTRINT UserData);
	// There is no RemoveInitialiseCallback because it is triggered and cleared before Main is called,
	// there should be no need to remove a callback before then.
	CORE_API static void AddTagCreationCallback(FTagCreationCallback Callback, UPTRINT UserData);
	CORE_API static void RemoveTagCreationCallback(FTagCreationCallback Callback);

	friend class ::FTagTrace;
};

} // UE::LLMPrivate

/** A convenient struct for gathering the fields needed to report in RegisterProjectTag */
struct FLLMTagInfo
{
	const TCHAR* Name;
	FName StatName;				// shows in the LLMFULL stat group
	FName SummaryStatName;		// shows in the LLM summary stat group
	int32 ParentTag = -1;
};

/** The public LLM interface. */
class FLowLevelMemTracker
{
public:

	/** Get the singleton, which makes sure that we always have a valid object */
	inline static FLowLevelMemTracker& Get()
	{
		if (TrackerInstance)
			return *TrackerInstance;
		else
			return Construct();
	}

	static CORE_API FLowLevelMemTracker& Construct();

	/**
	 * Return whether LLM is enabled - activated by the commandline and is tracking allocations.
	 * Also returns true early during startup before commandline has been processed.
	 */
	inline static bool IsEnabled();

	/**
	 * We always start up running, but if the commandline disables us, we will do it later after main
	 * (can't get the commandline early enough in a cross-platform way).
	 */
	CORE_API void ProcessCommandLine(const TCHAR* CmdLine);

	/** Return the total amount of memory being tracked. */
	CORE_API uint64 GetTotalTrackedMemory(ELLMTracker Tracker);

	/**
	 * Records the use of memory that was allocated for a pointer.
	 *
	 * @param Tracker Which tracker to use, the high-level default tracker for regular engine allocations, or the
	 *        low-level platform tracker for memory management systems.
	 * @param Ptr The pointer that was allocated.
	 * @param Size The size of the memory that was allocated for the pointer.
	 * @param DefaultTag The tag to use if there is no tag already in scope on the callstack.
	 * @param AllocType Type of allocation, FMalloc for regular allocations done through GMalloc, System for others.
	 * @param bTrackInMemPro Whether to pass the allocation/free information on to FMemProProfiler.
	 */
	CORE_API void OnLowLevelAlloc(ELLMTracker Tracker, const void* Ptr, uint64 Size, ELLMTag DefaultTag = ELLMTag::Untagged,
		ELLMAllocType AllocType = ELLMAllocType::None, bool bTrackInMemPro = true);
	CORE_API void OnLowLevelAlloc(ELLMTracker Tracker, const void* Ptr, uint64 Size, FName DefaultTag,
		ELLMAllocType AllocType = ELLMAllocType::None, bool bTrackInMemPro = true);

	/**
	 * Records the release of memory that was allocated for a pointer.
	 *
	 * @param Tracker Which tracker to use, the high-level default tracker for regular engine allocations, or the
	 *        low-level platform tracker for memory management systems.
	 * @param Ptr The pointer that was freed.
	 * @param AllocType Type of allocation, FMalloc for regular allocations done through GMalloc, System for others.
	 * @param bTrackInMemPro Whether to pass the allocation/free information on to FMemProProfiler.
	 */
	CORE_API void OnLowLevelFree(ELLMTracker Tracker, const void* Ptr,
		ELLMAllocType AllocType = ELLMAllocType::None, bool bTrackInMemPro = true);

	/**
	 * Records an allocation or release of memory by a system that is not directly associated with a single pointer.
	 *
	 * @param Tracker Which tracker to use, the high-level default tracker for regular engine allocations, or the
	 *        low-level platform tracker for memory management systems.
	 * @param DeltaMemory The amount of memory that was allocated (positive) or freed (negative)
	 * @param DefaultTag The tag to use if there is no tag already in scope on the callstack.
	 * @param AllocType Type of allocation, FMalloc for regular allocations done through GMalloc, System for others.
	 */
	CORE_API void OnLowLevelChangeInMemoryUse(ELLMTracker Tracker, int64 DeltaMemory, ELLMTag DefaultTag = ELLMTag::Untagged, ELLMAllocType AllocType = ELLMAllocType::None);
	CORE_API void OnLowLevelChangeInMemoryUse(ELLMTracker Tracker, int64 DeltaMemory, FName DefaultTag, ELLMAllocType AllocType = ELLMAllocType::None);

	/** Call if an allocation is moved in memory, such as in a defragger. */
	CORE_API void OnLowLevelAllocMoved(ELLMTracker Tracker, const void* Dest, const void* Source,
		ELLMAllocType AllocType = ELLMAllocType::None);

	/** Updates memory stats and optionally publishes them to observers. */
	CORE_API void UpdateStatsPerFrame(const TCHAR* LogName=nullptr);
	/** Updates memory stats. */
	CORE_API void Tick();

	/** Optionally set the amount of memory taken up before the game starts for executable and data segments .*/
	CORE_API void SetProgramSize(uint64 InProgramSize);

	/** console command handler */
	CORE_API bool Exec(const TCHAR* Cmd, FOutputDevice& Ar);

	/** Report whether the given TagSet is active, e.g. AssetTracking. */
	CORE_API bool IsTagSetActive(ELLMTagSet Set);

	/** For some tag sets, it's really useful to reduce threads, to attribute allocations to assets, for instance. */
	CORE_API bool ShouldReduceThreads();

	/** Get an opaque identifier for the top active tag for the given tracker. */
	CORE_API const UE::LLMPrivate::FTagData* GetActiveTagData(ELLMTracker Tracker, ELLMTagSet TagSet = ELLMTagSet::None);

	/** Register custom ELLMTags. */
	CORE_API void RegisterPlatformTag(int32 Tag, const TCHAR* Name, FName StatName, FName SummaryStatName, int32 ParentTag = -1);
	CORE_API void RegisterProjectTag(int32 Tag, const TCHAR* Name, FName StatName, FName SummaryStatName, int32 ParentTag = -1);
    
	/** Get all tags being tracked. */
	CORE_API TArray<const UE::LLMPrivate::FTagData*> GetTrackedTags(ELLMTagSet TagSet = ELLMTagSet::None);

	/** Get all tags being tracked by the given tracker. */
	CORE_API TArray<const UE::LLMPrivate::FTagData*> GetTrackedTags(ELLMTracker Tracker, ELLMTagSet TagSet = ELLMTagSet::None);

	CORE_API void GetTrackedTagsNamesWithAmount(TMap<FName, uint64>& TagsNamesWithAmount, ELLMTracker Tracker, ELLMTagSet TagSet);
	CORE_API void GetTrackedTagsNamesWithAmountFiltered(TMap<FName, uint64>& TagsNamesWithAmount, ELLMTracker Tracker, ELLMTagSet TagSet, TArray<FLLMTagSetAllocationFilter>& Filters);

	/** Look up the ELLMTag associated with the given display name. */
	CORE_API bool FindTagByName(const TCHAR* Name, uint64& OutTag, ELLMTagSet InTagSet = ELLMTagSet::None) const;

	/** Get the display name for the given ELLMTag. */
	CORE_API FName FindTagDisplayName(uint64 Tag) const;
	CORE_API FName FindPtrDisplayName(void* Ptr) const;

	/** Get the display name for the given FTagData. */
	CORE_API FName GetTagDisplayName(const UE::LLMPrivate::FTagData* TagData) const;

	/** Get the path name for the given FTagData from a chain of its parents' display names. */
	CORE_API FString GetTagDisplayPathName(const UE::LLMPrivate::FTagData* TagData) const;
	CORE_API void GetTagDisplayPathName(const UE::LLMPrivate::FTagData* TagData,
		FStringBuilderBase& OutPathName, int32 MaxLen=-1) const;

	/** Get the unique identifier name for the given FTagData. */
	CORE_API FName GetTagUniqueName(const UE::LLMPrivate::FTagData* TagData) const;

	/** Return the TagData that is the parent scope of the given TagData, or nullptr if no parent. */
	CORE_API const UE::LLMPrivate::FTagData* GetTagParent(const UE::LLMPrivate::FTagData* TagData) const;

	/** Return true if and only if the TagData is an ELLMTag or a custom-declared platform or project ELLMTag. */
	CORE_API bool GetTagIsEnumTag(const UE::LLMPrivate::FTagData* TagData) const;

	/**
	 * Return the ELLMTag of closest parent (including possibly TagData) which has true==GetTagIsEnumTag
	 * For FName tags with no ELLMTag parent, they will return ELLMTag::CustomName.
	 */
	CORE_API ELLMTag GetTagClosestEnumTag(const UE::LLMPrivate::FTagData* TagData) const;

	/** Get the amount of memory for an ELLMTag from the given tracker. */
	CORE_API int64 GetTagAmountForTracker(ELLMTracker Tracker, ELLMTag Tag, UE::LLM::ESizeParams SizeParams = UE::LLM::ESizeParams::Default);

	/** Get the amount of memory for a FTagData from the given tracker. */
	CORE_API int64 GetTagAmountForTracker(ELLMTracker Tracker, const UE::LLMPrivate::FTagData* TagData, UE::LLM::ESizeParams SizeParams = UE::LLM::ESizeParams::Default);

    CORE_API int64 GetTagAmountForTracker(ELLMTracker Tracker, FName Tag, ELLMTagSet TagSet, UE::LLM::ESizeParams SizeParams = UE::LLM::ESizeParams::Default);

	/** Set the amount of memory for an ELLMTag for a given tracker and optionally update the total tracked memory. */
	CORE_API void SetTagAmountForTracker(ELLMTracker Tracker, ELLMTag Tag, int64 Amount, bool bAddToTotal);

	/** Set the amount of memory for an tag name for a given tracker and optionally update the total tracked memory. */
	CORE_API void SetTagAmountForTracker(ELLMTracker Tracker, FName Tag, ELLMTagSet TagSet, int64 Amount, bool bAddToTotal);
	
	/** Dump the display name of the current TagData for the given tracker to the output. */
	CORE_API uint64 DumpTag(ELLMTracker Tracker, const char* FileName, int LineNumber);

	/** Publishes the active LLM stats in the active frame, useful for single targeted LLM snapshots. */
	CORE_API void PublishDataSingleFrame();

	enum class EDumpFormat
	{
		PlainText,
		CSV,
	};
	CORE_API void DumpToLog(EDumpFormat DumpFormat = EDumpFormat::PlainText, FOutputDevice* OutputDevice = nullptr, UE::LLM::ESizeParams SizeParams = UE::LLM::ESizeParams::Default, ELLMTagSet TagSet = ELLMTagSet::None);

	CORE_API void OnPreFork();

	CORE_API bool IsInitialized() const;
	CORE_API bool IsConfigured() const;

	/**
	 * Initialize the data required for tracking allocations if it has not already been done.
	 * This can occur after construction of the singleton FLowLevelMemTracker. 
	 * This function is called automatically by any LLM function that requires it, but can also be triggered
	 * by external systems that need to carefully manage static initialization order.
	 * BootstrapInitialize does NOT make GetTagUniqueName available; that requires FinishInitialize.
	 */
	CORE_API void BootstrapInitialise();

	/**
	 * Finish initialization if not already done. Calls BootstrapInitialise if necessary. After being called,
	 * all functions in LLM including those returning FNames are available.
	 * This function should be called as soon as possible after FNames are available so that other tracking
	 * systems reliant on LLM's FName data can start tracking as soon as possible.
	 */
	CORE_API void FinishInitialise();

private:
	UE::LLMPrivate::FLLMGlobals* GetGlobals();
	const UE::LLMPrivate::FLLMGlobals* GetGlobals() const;
	/**
	 * A function that returns IsEnabled(), but also enters a lock if current state is NotYetKnown and therefore might
	 * change if the main thread calls ProcessCommandLine and commandline specifies that LLM should disable itself.
	 * The lock is only needed on platforms with PLATFORM_HAS_MULTITHREADED_PREMAIN; other platforms are not
	 * multithreaded until after ProcessCommandLine is called. This function is only necessary on handlers for new
	 * and delete (OnLowLevelAlloc, etc), since those are the only functions that can be called from other threads
	 * before ProcessCommandLine.
	 */
	inline static bool TryEnterEnabled(UE::LLMPrivate::FEnableStateScopeLock& ScopeLock);

	static CORE_API FLowLevelMemTracker* TrackerInstance;
	static CORE_API UE::LLMPrivate::EEnabled EnabledState;

	friend UE::LLMPrivate::FLLMGlobals;
};

/** LLM scope for tracking memory. */
class FLLMScope
{
public:
	CORE_API explicit FLLMScope(FName TagName, bool bIsStatTag, ELLMTagSet InTagSet, ELLMTracker InTracker, bool bOverride = true);
	CORE_API explicit FLLMScope(ELLMTag TagEnum, bool bIsStatTag, ELLMTagSet InTagSet, ELLMTracker InTracker, bool bOverride = true);
	CORE_API explicit FLLMScope(const UE::LLMPrivate::FTagData* TagData, bool bIsStatTag, ELLMTagSet InTagSet, ELLMTracker InTracker, bool bOverride = true);
	~FLLMScope()
	{
		if (bEnabled)
		{
			Destruct();
		}
	}

protected:
	CORE_API void Destruct();
	AUTORTFM_DISABLE void DestructInTheOpen();
	template <typename ShouldEnableType, typename OnEnabledType>
	void Init(ELLMTagSet InTagSet, ELLMTracker InTracker, bool bOverride,
		ShouldEnableType&& ShouldEnable, OnEnabledType&& OnEnable);

	// All fields other than bEnabled are not initialized by constructor unless bEnabled is true
	ELLMTracker Tracker;
	bool bEnabled = false;  // Needs to be a uint8 rather than a bitfield to minimize ALU operations
	bool bPushedCodeOrContent = false;
	ELLMTagSet TagSet;
};

/**
 * Provides arguments for TagConstruction that are passed through a LLM_SCOPE_DYNAMIC macro and should
 * only be evaluated on the first LLM_SCOPE_DYNAMIC call.
 */
class ILLMDynamicTagConstructor
{
public:
	virtual ~ILLMDynamicTagConstructor() = default;
	virtual FString GetStatName() const
	{
		return FString();
	}
	virtual bool NeedsStatConstruction() const
	{
		return true;
	}
};

/** ILLMDynamicTagConstructor that provides a string to pass to CreateMemoryStatId. */
class FLLMDynamicTagConstructorStatString : public ILLMDynamicTagConstructor
{
public:
	FLLMDynamicTagConstructorStatString(FString InStatName) : StatName(MoveTemp(InStatName)) {}
	virtual FString GetStatName() const override { return StatName; }
	FString StatName;
};

/**
 * LLM scope for tracking memory for a dynamically-created tag. Like some normal tags, dynamically-created tags
 * are created by Name, rather than a compile-time constant like ELLMTag or LLM_DECLARE_TAG, and they are
 * created on demand by the first scope that uses them.
 * Unlike normal tags, dynamically-created tags also provide the ability to create stats when first called.
 */
class FLLMScopeDynamic
{
public:
	FLLMScopeDynamic(ELLMTracker InTracker, ELLMTagSet InTagSet)
	{
		if (FLowLevelMemTracker::IsEnabled())
		{
			Init(InTracker, InTagSet);
		}
	}
	bool IsEnabled() const { return bEnabled; }
	CORE_API bool TryFindTag(FName UniqueName); // Set name separately so the calculation of name can be skipped if TagSet is disabled
	CORE_API bool TryAddTagAndActivate(FName UniqueName, const ILLMDynamicTagConstructor& Constructor);
	CORE_API void Activate();

	~FLLMScopeDynamic()
	{
		if (bEnabled)
		{
			Destruct();
		}
	}

protected:
	CORE_API void Init(ELLMTracker InTracker, ELLMTagSet InTagSet);
	CORE_API void Destruct();
	AUTORTFM_DISABLE void DestructInTheOpen();

	// All fields other than bEnabled are not initialized by constructor unless bEnabled is true
	const UE::LLMPrivate::FTagData* TagData;
	const UE::LLMPrivate::FTagData* CodeOrContentTagData;
	ELLMTracker Tracker;
	bool bEnabled = false; // Needs to be a uint8 rather than a bitfield to minimize ALU operations
	ELLMTagSet TagSet;
};

/** LLM scope for pausing LLM (disables the allocation hooks). */
class FLLMPauseScope
{
public:
	CORE_API FLLMPauseScope(FName TagName, bool bIsStatTag, uint64 Amount, ELLMTracker TrackerToPause, ELLMAllocType InAllocType);
	CORE_API FLLMPauseScope(ELLMTag TagEnum, bool bIsStatTag, uint64 Amount, ELLMTracker TrackerToPause, ELLMAllocType InAllocType);
	~FLLMPauseScope()
	{
		if (bEnabled)
		{
			Destruct();
		}
	}
protected:
	CORE_API void Init(FName TagName, ELLMTag EnumTag, bool bIsEnumTag, bool bIsStatTag, uint64 Amount,
		ELLMTracker TrackerToPause, ELLMAllocType InAllocType);
	CORE_API void Destruct();
	AUTORTFM_DISABLE void DestructInTheOpen();

	// All fields other than bEnabled are not initialized by constructor unless bEnabled is true
	ELLMTracker PausedTracker;
	ELLMAllocType AllocType;
	bool bEnabled = false;
};

/** LLM Scope to clear top tag while in scope */
class FLLMClearScope : public FLLMScope
{
public:
	CORE_API FLLMClearScope(ELLMTagSet InTagSet, ELLMTracker InTracker);
};

/** LLM scope for inheriting tag from the given address. */
class FLLMScopeFromPtr
{
public:
	CORE_API FLLMScopeFromPtr(void* Ptr, ELLMTracker Tracker);
	~FLLMScopeFromPtr()
	{
		if (bEnabled)
		{
			Destruct();
		}
	}

protected:
	CORE_API void Destruct();
	AUTORTFM_DISABLE void DestructInTheOpen();

	// All fields other than bEnabled are not initialized by constructor unless bEnabled is true
	ELLMTracker Tracker;
	bool bSetEnabled[static_cast<int32>(ELLMTagSet::Max)];
	bool bEnabled = false;
};

/** Global instances to provide information about a tag to LLM. */
class FLLMTagDeclaration
{
public:
	CORE_API FLLMTagDeclaration(const TCHAR* InCPPName, const FName InDisplayName=NAME_None, FName InParentTagName = NAME_None, FName InStatName = NAME_None, FName InSummaryStatName = NAME_None, ELLMTagSet TagSet = ELLMTagSet::None);
	FName GetUniqueName() const { return UniqueName; }

	typedef void (*FCreationCallback)(FLLMTagDeclaration&);
protected:

	static CORE_API void AddCreationCallback(FCreationCallback InCallback);
	static CORE_API void ClearCreationCallbacks();
	static CORE_API TArrayView<FCreationCallback> GetCreationCallbacks();
	static CORE_API FLLMTagDeclaration* GetList();

	CORE_API void Register();

protected:
	CORE_API void ConstructUniqueName();

	const TCHAR* CPPName;
	FName UniqueName;
	FName DisplayName;
	FName ParentTagName;
	FName StatName;
	FName SummaryStatName;
	ELLMTagSet TagSet;
	FLLMTagDeclaration* Next = nullptr;

	friend class UE::LLMPrivate::FLLMGlobals;
};

/** Used to filter Tag allocations specifying a Name that matches in a TagSet, or TagSet alone or just by Name. */
struct FLLMTagSetAllocationFilter
{
	/** Tag name to match */
	FName Name;
	/** Tag set to match */
	ELLMTagSet TagSet;
};

inline bool FLowLevelMemTracker::IsEnabled()
{
	return EnabledState != UE::LLMPrivate::EEnabled::Disabled;
}

#else

#define LLM(...)
#define LLM_IF_ENABLED(...)
#define LLM_IS_ENABLED() false
#define LLM_SCOPE(...)
#define LLM_SCOPE_DYNAMIC(...)
#define LLM_TAGSET_SCOPE(...)
#define LLM_SCOPE_BYNAME(...)
#define LLM_SCOPE_BYTAG(...)
#define LLM_SCOPE_RENDER_RESOURCE(...)
#define LLM_PLATFORM_SCOPE(...)
#define LLM_PLATFORM_SCOPE_BYNAME(...)
#define LLM_PLATFORM_SCOPE_BYTAG(...)
#define LLM_SCOPE_CLEAR()
#define LLM_TAGSET_SCOPE_CLEAR(...)
#define LLM_REALLOC_SCOPE(...)
#define LLM_REALLOC_PLATFORM_SCOPE(...)
#define LLM_SCOPED_PAUSE_TRACKING(...)
#define LLM_SCOPED_PAUSE_TRACKING_FOR_TRACKER(...)
#define LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT(...)
#define LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(...)
#define LLM_DUMP_TAG()
#define LLM_DUMP_PLATFORM_TAG()
#define LLM_DEFINE_TAG(...)
#define LLM_DEFINE_STATIC_TAG(...)
#define LLM_DECLARE_TAG(...)
#define LLM_DECLARE_TAG_API(...)
#define LLM_DEFINE_BOOTSTRAP_TAG(...)
#define LLM_SCOPE_BY_BOOTSTRAP_TAG(...)
#define LLM_DECLARE_BOOTSTRAP_TAG(...) 
#define LLM_DECLARE_BOOTSTRAP_TAG_API(...)

#endif		// #if ENABLE_LOW_LEVEL_MEM_TRACKER