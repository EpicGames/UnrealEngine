// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// This header configures the compile-time settings used by LLM, based on defines from the compilation environment.
// This header can be read from c code, so it should not include any c++ code, or any headers that include c++ code.

#include "Misc/Build.h"

#ifndef ALLOW_LOW_LEVEL_MEM_TRACKER_IN_TEST
	#define ALLOW_LOW_LEVEL_MEM_TRACKER_IN_TEST 1	// enabled in Test by default to facilitate LLM captures in auto perf testing
#endif

// LLM is usually not enabled in programs, but some programs want it (e.g. unittests).
// Define LLM_ALLOWED_IN_APPLICATION_TYPE in target.cs if needed in a program or other non-engine application type.
#ifndef LLM_ALLOWED_IN_APPLICATION_TYPE
	#define LLM_ALLOWED_IN_APPLICATION_TYPE WITH_ENGINE
#endif

// LLM_ENABLED_IN_CONFIG can be defined here or in the build environment only.
// It cannot be defined in platform header files because it is included in c-language compilation units that
// cannot include those headers.
// When locally instrumenting, add your own definition of LLM_ENABLED_IN_CONFIG in build environment or this header.

#ifndef LLM_ENABLED_IN_CONFIG 
	#define LLM_ENABLED_IN_CONFIG ( \
		!UE_BUILD_SHIPPING && (!UE_BUILD_TEST || ALLOW_LOW_LEVEL_MEM_TRACKER_IN_TEST) && \
		LLM_ALLOWED_IN_APPLICATION_TYPE \
	)
#endif
