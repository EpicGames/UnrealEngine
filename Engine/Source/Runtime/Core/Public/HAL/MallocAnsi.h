// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/MemoryBase.h"
#include "HAL/UnrealMemory.h"

/**
 * AnsiMalloc API: Malloc
 * 
 * Functions in the AnsiMalloc API provide allocations from c++ or operating system allocations functions
 * without going through an intermediate heap manager.
 * These allocations are tracked in Unreal's memory trackers.
 * As usual with allocation functions, AnsiRealloc and AnsiFree should only be called on nullptr or on pointers
 * returned from AnsiMalloc or AnsiRealloc.
 */
CORE_API void* AnsiMalloc(SIZE_T Size, uint32 Alignment);
/** AnsiMalloc API: Realloc. @see AnsiMalloc. */
CORE_API void* AnsiRealloc(void* Ptr, SIZE_T NewSize, uint32 Alignment);
/** AnsiMalloc API: Free. @see AnsiMalloc. */
CORE_API void AnsiFree(void* Ptr);

/**
 * AnsiMallocUntracked API: Malloc
 *
 * Functions in the AnsiMallocUntracked API provide allocations from c++ or operating system allocations functions
 * without going through an intermediate heap manager.
 * Unlike AnsiMalloc API, these allocations are NOT tracked in Unreal's memory trackers.
 * As usual with allocation functions, AnsiReallocUntracked and AnsiFreeUntracked should only be called on nullptr or
 * on pointers returned from AnsiMallocUntracked or AnsiReallocUntracked.
 * 
 * This API exists only for cases where attempting to track the allocations causes a conflict. We prefer all
 * allocations to be tracked so memory usage can be analyzed, so if no conflict exists or it can be worked around,
 * callers should use AnsiMalloc API instead.
 * 
 * Callsites using the AnsiMallocUntracked API should document the conflict.
 */
CORE_API void* AnsiMallocUntracked(SIZE_T Size, uint32 Alignment);
/** AnsiMallocUntracked API: Realloc. @see AnsiMallocUntracked. */
CORE_API void* AnsiReallocUntracked(void* Ptr, SIZE_T NewSize, uint32 Alignment);
/** AnsiMallocUntracked API: Free. @see AnsiMallocUntracked. */
CORE_API void AnsiFreeUntracked(void* Ptr);

//
// ANSI C memory allocator.
//
class FMallocAnsi final
	: public FMalloc
{
	
public:
	/**
	 * Constructor enabling low fragmentation heap on platforms supporting it.
	 */
	FMallocAnsi();

	// FMalloc interface.
	virtual void* Malloc( SIZE_T Size, uint32 Alignment ) override;

	virtual void* TryMalloc(SIZE_T Size, uint32 Alignment) override;

	virtual void* Realloc( void* Ptr, SIZE_T NewSize, uint32 Alignment ) override;

	virtual void* TryRealloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override;

	virtual void Free( void* Ptr ) override;

	virtual bool GetAllocationSize(void *Original, SIZE_T &SizeOut) override;

	/**
	 * Returns if the allocator is guaranteed to be thread-safe and therefore
	 * doesn't need a unnecessary thread-safety wrapper around it.
	 *
	 * @return true as we're using system allocator
	 */
	virtual bool IsInternallyThreadSafe() const override;

	/**
	 * Validates the allocator's heap
	 */
	virtual bool ValidateHeap() override;

	virtual const TCHAR* GetDescriptiveName() override { return TEXT("ANSI"); }
};
