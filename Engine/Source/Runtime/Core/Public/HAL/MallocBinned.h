// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "HAL/LowLevelMemStats.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/MallocJemalloc.h"
#include "HAL/MemoryBase.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PlatformMutex.h"
#include "Math/UnrealMathUtility.h"
#include "Stats/Stats.h"

struct FGenericMemoryStats;

#define MEM_TIME(st)

//#define USE_LOCKFREE_DELETE
#define USE_INTERNAL_LOCKS
#if USE_CACHE_FREED_OS_ALLOCS
//#define CACHE_FREED_OS_ALLOCS
#endif

#ifndef USE_OS_SMALL_BLOCK_ALLOC
#define USE_OS_SMALL_BLOCK_ALLOC PLATFORM_IOS
#endif //USE_OS_SMALL_BLOCK_ALLOC

#ifndef USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
#define USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS (USE_OS_SMALL_BLOCK_ALLOC && PLATFORM_IOS && 0)
#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS

#ifdef USE_INTERNAL_LOCKS
//#   define USE_COARSE_GRAIN_LOCKS
#endif

#if defined USE_LOCKFREE_DELETE
#	define USE_INTERNAL_LOCKS
#	define USE_COARSE_GRAIN_LOCKS
#endif

#if defined CACHE_FREED_OS_ALLOCS
	#define MAX_CACHED_OS_FREES (64)
	#define MAX_CACHED_OS_FREES_BYTE_LIMIT (64*1024*1024)
#endif

#if defined USE_INTERNAL_LOCKS && !defined USE_COARSE_GRAIN_LOCKS
#	define USE_FINE_GRAIN_LOCKS
#endif

typedef int64 BINNED_STAT_TYPE;

//when modifying the global allocator stats, if we are using COARSE locks, then all callsites for stat modification are covered by the allocator-wide access guard. Thus the stats can be modified directly.
//If we are using FINE locks, then we must modify the stats through atomics as the locks are either not actually covering the stat callsites, or are locking specific table locks which is not sufficient for stats.
#if STATS
#	ifdef USE_COARSE_GRAIN_LOCKS
#		define BINNED_STAT BINNED_STAT_TYPE
#		define BINNED_INCREMENT_STATCOUNTER(counter) (++(counter))
#		define BINNED_DECREMENT_STATCOUNTER(counter) (--(counter))
#		define BINNED_ADD_STATCOUNTER(counter, value) ((counter) += (value))
#		define BINNED_PEAK_STATCOUNTER(PeakCounter, CompareVal) ((PeakCounter) = FMath::Max((PeakCounter), (CompareVal)))
#	else
#		define BINNED_STAT volatile BINNED_STAT_TYPE
#		define BINNED_INCREMENT_STATCOUNTER(counter) (FPlatformAtomics::InterlockedIncrement(&(counter)))
#		define BINNED_DECREMENT_STATCOUNTER(counter) (FPlatformAtomics::InterlockedDecrement(&(counter)))
#		define BINNED_ADD_STATCOUNTER(counter, value) (FPlatformAtomics::InterlockedAdd(&counter, (value)))
#		define BINNED_PEAK_STATCOUNTER(PeakCounter, CompareVal)	{																												\
																	BINNED_STAT_TYPE NewCompare;																							\
																	BINNED_STAT_TYPE NewPeak;																								\
																	do																											\
																	{																											\
																		NewCompare = (PeakCounter);																				\
																		NewPeak = FMath::Max((PeakCounter), (CompareVal));														\
																	}																											\
																	while (FPlatformAtomics::InterlockedCompareExchange(&(PeakCounter), NewPeak, NewCompare) != NewCompare);	\
																}
#	endif
#else
#	define BINNED_STAT BINNED_STAT_TYPE
#	define BINNED_INCREMENT_STATCOUNTER(counter)
#	define BINNED_DECREMENT_STATCOUNTER(counter)
#	define BINNED_ADD_STATCOUNTER(counter, value)
#	define BINNED_PEAK_STATCOUNTER(PeakCounter, CompareVal)
#endif

/** Malloc binned allocator specific stats. */
DECLARE_MEMORY_STAT_EXTERN(TEXT("Binned Os Current"),		STAT_Binned_OsCurrent,STATGROUP_MemoryAllocator, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Binned Os Peak"),			STAT_Binned_OsPeak,STATGROUP_MemoryAllocator, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Binned Waste Current"),	STAT_Binned_WasteCurrent,STATGROUP_MemoryAllocator, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Binned Waste Peak"),		STAT_Binned_WastePeak,STATGROUP_MemoryAllocator, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Binned Used Current"),		STAT_Binned_UsedCurrent,STATGROUP_MemoryAllocator, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Binned Used Peak"),		STAT_Binned_UsedPeak,STATGROUP_MemoryAllocator, CORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Binned Current Allocs"),	STAT_Binned_CurrentAllocs,STATGROUP_MemoryAllocator, CORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Binned Total Allocs"),		STAT_Binned_TotalAllocs,STATGROUP_MemoryAllocator, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Binned Slack Current"),	STAT_Binned_SlackCurrent,STATGROUP_MemoryAllocator, CORE_API);

#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS && ENABLE_LOW_LEVEL_MEM_TRACKER
DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Nano Malloc Pages Current"), STAT_Binned_NanoMallocPages_Current,STATGROUP_LLMPlatform, CORE_API);
DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Nano Malloc Pages Peak"), STAT_Binned_NanoMallocPages_Peak,STATGROUP_LLMPlatform, CORE_API);
DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Nano Malloc Pages Waste Current"), STAT_Binned_NanoMallocPages_WasteCurrent,STATGROUP_LLMPlatform, CORE_API);
DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("Nano Malloc Pages Waste Peak"),STAT_Binned_NanoMallocPages_WastePeak,STATGROUP_LLMPlatform, CORE_API);
#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS


//
// Optimized virtual memory allocator.
//
class FMallocBinned : public FMalloc
{
	struct Private;

private:

	// Counts.
	enum { POOL_COUNT = 41 };

	/** Maximum allocation for the pooled allocator */
	enum
	{
		EXTENDED_PAGE_POOL_ALLOCATION_COUNT = 2,
		MAX_POOLED_ALLOCATION_SIZE = 32768 + 1,
	};
	
	// Forward declares.
	struct FFreeMem;
	struct FPoolInfo;
	struct FPoolTable;
	struct PoolHashBucket;

#ifdef CACHE_FREED_OS_ALLOCS
	/**  */
	struct FFreePageBlock
	{
		void*				Ptr;
		SIZE_T				ByteSize;

		FFreePageBlock() 
		{
			Ptr = nullptr;
			ByteSize = 0;
		}
	};
#endif

	/** Pool table. */
	struct FPoolTable
	{
		FPoolInfo*			FirstPool;
		FPoolInfo*			ExhaustedPool;
		uint32				BlockSize;
#ifdef USE_FINE_GRAIN_LOCKS
		UE::FPlatformRecursiveMutex CriticalSection;
#endif
#if STATS
		/** Number of currently active pools */
		uint32				NumActivePools;

		/** Largest number of pools simultaneously active */
		uint32				MaxActivePools;

		/** Number of requests currently active */
		uint32				ActiveRequests;

		/** High watermark of requests simultaneously active */
		uint32				MaxActiveRequests;

		/** Minimum request size (in bytes) */
		uint32				MinRequest;

		/** Maximum request size (in bytes) */
		uint32				MaxRequest;

		/** Total number of requests ever */
		uint64				TotalRequests;

		/** Total waste from all allocs in this table */
		uint64				TotalWaste;
#endif
		FPoolTable()
			: FirstPool(nullptr)
			, ExhaustedPool(nullptr)
			, BlockSize(0)
#if STATS
			, NumActivePools(0)
			, MaxActivePools(0)
			, ActiveRequests(0)
			, MaxActiveRequests(0)
			, MinRequest(0)
			, MaxRequest(0)
			, TotalRequests(0)
			, TotalWaste(0)
#endif
		{

		}
	};

	uint64 TableAddressLimit;

#ifdef USE_LOCKFREE_DELETE
	/** We can't call the constructor to TLockFreePointerList in the BinnedMalloc constructor
	* as it attempts to allocate memory. We push this back and initialize it later but we 
	* set aside the memory before hand
	*/
	uint8							PendingFreeListMemory[sizeof(TLockFreePointerList<void>)];
	TLockFreePointerList<void>*		PendingFreeList;
	TArray<void*>					FlushedFrees;
	bool							bFlushingFrees;
	bool							bDoneFreeListInit;
#endif

	UE::FPlatformRecursiveMutex AccessGuard;

	// PageSize dependent constants
	uint64 MaxHashBuckets;
	uint64 MaxHashBucketBits;
	uint64 MaxHashBucketWaste;
	uint64 MaxBookKeepingOverhead;
	/** Shift to get the reference from the indirect tables */
	uint64 PoolBitShift;
	uint64 IndirectPoolBitShift;
	uint64 IndirectPoolBlockSize;
	/** Shift required to get required hash table key. */
	uint64 HashKeyShift;
	/** Used to mask off the bits that have been used to lookup the indirect table */
	uint64 PoolMask;
	uint64 BinnedSizeLimit;
	uint64 BinnedOSTableIndex;

	// Variables.
	FPoolTable  PoolTable[POOL_COUNT];
	FPoolTable	OsTable;
	FPoolTable	PagePoolTable[EXTENDED_PAGE_POOL_ALLOCATION_COUNT];
	FPoolTable* MemSizeToPoolTable[MAX_POOLED_ALLOCATION_SIZE + EXTENDED_PAGE_POOL_ALLOCATION_COUNT];
	
	PoolHashBucket* HashBuckets;
	PoolHashBucket* HashBucketFreeList;
	
	uint32		PageSize;
	
#ifdef CACHE_FREED_OS_ALLOCS
	FFreePageBlock	FreedPageBlocks[MAX_CACHED_OS_FREES];
	uint32			FreedPageBlocksNum;
	uint32			CachedTotal;
#endif

#if STATS
	BINNED_STAT		OsCurrent;
	BINNED_STAT		OsPeak;
	BINNED_STAT		WasteCurrent;
	BINNED_STAT		WastePeak;
	BINNED_STAT		UsedCurrent;
	BINNED_STAT		UsedPeak;
	BINNED_STAT		CurrentAllocs;
	BINNED_STAT		TotalAllocs;
	/** OsCurrent - WasteCurrent - UsedCurrent. */
	BINNED_STAT		SlackCurrent;
	double		MemTime;
	
#if USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
	BINNED_STAT		NanoMallocPagesCurrent;
	BINNED_STAT		NanoMallocPagesPeak;
	BINNED_STAT		NanoMallocPagesWaste;
	BINNED_STAT		NanoMallocWastePagesPeak;
#endif //USE_OS_SMALL_BLOCK_GRAB_MEMORY_FROM_OS
	
#endif //STATS

#if USE_OS_SMALL_BLOCK_ALLOC
	bool			bNanoMallocAvailable;
#endif

public:
	// FMalloc interface.
	// InPageSize - First parameter is page size, all allocs from BinnedAllocFromOS() MUST be aligned to this size
	// AddressLimit - Second parameter is estimate of the range of addresses expected to be returns by BinnedAllocFromOS(). Binned
	// Malloc will adjust its internal structures to make lookups for memory allocations O(1) for this range. 
	// It is ok to go outside this range, lookups will just be a little slower
	FMallocBinned(uint32 InPageSize, uint64 AddressLimit);

	virtual void InitializeStatsMetadata() override;

	virtual ~FMallocBinned();

	/**
	 * Returns if the allocator is guaranteed to be thread-safe and therefore
	 * doesn't need a unnecessary thread-safety wrapper around it.
	 */
	virtual bool IsInternallyThreadSafe() const override;

	/** 
	 * Malloc
	 */
	virtual void* Malloc( SIZE_T Size, uint32 Alignment ) override;

	/** 
	 * Realloc
	 */
	virtual void* Realloc( void* Ptr, SIZE_T NewSize, uint32 Alignment ) override;

	/** 
	 * Free
	 */
	virtual void Free( void* Ptr ) override;

	/**
	 * If possible determine the size of the memory allocated at the given address
	 *
	 * @param Original - Pointer to memory we are checking the size of
	 * @param SizeOut - If possible, this value is set to the size of the passed in pointer
	 * @return true if succeeded
	 */
	virtual bool GetAllocationSize(void *Original, SIZE_T &SizeOut) override;

	virtual SIZE_T QuantizeSize(SIZE_T Count, uint32 Alignment) override;

	/**
	 * Validates the allocator's heap
	 */
	virtual bool ValidateHeap() override;

	/** Called once per frame, gathers and sets all memory allocator statistics into the corresponding stats. MUST BE THREAD SAFE. */
	virtual void UpdateStats() override;

	/** Writes allocator stats from the last update into the specified destination. */
	virtual void GetAllocatorStats( FGenericMemoryStats& out_Stats ) override;

	/**
	 * Dumps allocator stats to an output device. Subclasses should override to add additional info
	 *
	 * @param Ar	[in] Output device
	 */
	virtual void DumpAllocatorStats( class FOutputDevice& Ar ) override;

	virtual const TCHAR* GetDescriptiveName() override;
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#	include "HAL/CriticalSection.h"
#endif
