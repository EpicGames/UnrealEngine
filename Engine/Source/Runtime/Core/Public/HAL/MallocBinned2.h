// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Async/UniqueLock.h"
#include "AutoRTFM.h"
#include "HAL/Allocators/CachedOSPageAllocator.h"
#include "HAL/Allocators/CachedOSVeryLargePageAllocator.h"
#include "HAL/Allocators/PooledVirtualMemoryAllocator.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/MallocBinnedCommon.h"
#include "HAL/PlatformMutex.h"
#include "HAL/UnrealMemory.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Fork.h"
#include "Templates/Atomic.h"


#define UE_MB2_MAX_CACHED_OS_FREES (64)
#define UE_MB2_MAX_CACHED_OS_FREES_BYTE_LIMIT (64*1024*1024)

#define UE_MB2_LARGE_ALLOC					65536		// Alignment of OS-allocated pointer - pool-allocated pointers will have a non-aligned pointer

#if AGGRESSIVE_MEMORY_SAVING
#	define UE_MB2_MAX_SMALL_POOL_SIZE		(13104)		// Maximum bin size in SmallBinSizesInternal in cpp file
#	define UE_MB2_SMALL_POOL_COUNT			48
#else
#	define UE_MB2_MAX_SMALL_POOL_SIZE		(32768-16)	// Maximum bin size in SmallBinSizesInternal in cpp file
#	define UE_MB2_SMALL_POOL_COUNT			51
#endif

// If we are emulating forking on a windows server or are a linux server, enable support for avoiding dirtying pages owned by the parent. 
#ifndef BINNED2_FORK_SUPPORT
#	define BINNED2_FORK_SUPPORT (UE_SERVER && (PLATFORM_UNIX || DEFAULT_SERVER_FAKE_FORKS))
#endif

#define UE_MB2_ALLOCATOR_STATS				UE_MBC_ALLOCATOR_STATS

// Double-free detection for small pool allocations. Writes a pointer-derived canary into freed
// bins and checks for it on subsequent frees. Near-zero overhead (same cache line as free list
// writes). Can be disabled if profiling shows unexpected impact.
#ifndef UE_MB2_DOUBLE_FREE_DETECTION
#	define UE_MB2_DOUBLE_FREE_DETECTION		0
#endif

// Canary value used in FFreeBlock
// A constant value unless we're compiled with fork support in which case there are two values identifying whether the page
// was allocated pre- or post-fork
enum class EBlockCanary : uint8
{
	Zero = 0x0, // Not clear why this is needed by FreeBundles
#if BINNED2_FORK_SUPPORT
	PreFork = 0xb7,
	PostFork = 0xca,
#else
	Value = 0xe3
#endif
};


//
// Optimized virtual memory allocator.
//
class FMallocBinned2 : public TMallocBinnedCommon<FMallocBinned2, UE_MB2_SMALL_POOL_COUNT, UE_MB2_MAX_SMALL_POOL_SIZE>
{
	struct FFreeBlock;

public:
	struct FPoolTable;

	struct FPoolInfo
	{
		enum class ECanary : uint16
		{
			Unassigned = 0x3941,
			FirstFreeBlockIsOSAllocSize = 0x17ea,
			FirstFreeBlockIsPtr = 0xf317
		};

		uint16      Taken;          // Number of allocated elements in this pool, when counts down to zero can free the entire pool	
		ECanary     Canary;         // See ECanary
		uint32      AllocSize;      // Number of bytes allocated
		FFreeBlock* FirstFreeBlock; // Pointer to first free memory in this pool or the OS Allocation Size in bytes if this allocation is not binned
		FPoolInfo*  Next;           // Pointer to next pool
		FPoolInfo** PtrToPrevNext;  // Pointer to whichever pointer points to this pool

		FPoolInfo();

		void CheckCanary(ECanary ShouldBe) const;
		void SetCanary(ECanary ShouldBe, bool bPreexisting, bool bGuaranteedToBeNew);

		bool HasFreeBin() const;
		void* AllocateBin(uint32 BinSize);

		SIZE_T GetOSRequestedBytes() const;
		SIZE_T GetOsAllocatedBytes() const;
		void SetOSAllocationSizes(SIZE_T InRequestedBytes, UPTRINT InAllocatedBytes);

		void Link(FPoolInfo*& PrevNext);
		void Unlink();

	private:
		void ExhaustPoolIfNecessary();
	};

private:
	// Forward declares.
	struct Private;

	/** Information about a piece of free memory. */
	struct FFreeBlock
	{
		inline FFreeBlock(uint32 InPageSize, uint16 InBinSize)
			: NextFreeBlock(0)
		{
			NumFreeBins = InPageSize / InBinSize;
			if (NumFreeBins * InBinSize + sizeof(FFreeBlock) > InPageSize)
			{
				NumFreeBins--;
			}
			check(NumFreeBins * InBinSize + sizeof(FFreeBlock) <= InPageSize);
		}

		FORCEINLINE uint32 GetNumFreeBins() const
		{
			return NumFreeBins;
		}

		FORCEINLINE FFreeBlock* GetNextFreeBlock() const
		{
			return (FFreeBlock*)NextFreeBlock;
		}

		FORCEINLINE void SetNextFreeBlock(FFreeBlock* Next)
		{
			NextFreeBlock = uint64(Next);
		}

		inline void* AllocateBin(uint32 BinSize)
		{
			--NumFreeBins;
			// this is a pointer to a whole 64KB block
			if (IsAligned(this, UE_MB2_LARGE_ALLOC))
			{
				return (uint8*)this + UE_MB2_LARGE_ALLOC - (NumFreeBins + 1) * BinSize;
			}

			// this is a pointer to a single bin within the 64KB block
			return (uint8*)this + (NumFreeBins * BinSize);
		}

		constexpr static uint32 MaxBitsForFreeBins = 32 - FMath::CountLeadingZeros((UE_MB2_LARGE_ALLOC - 16) / 16);	// max number of free bins in the 64kb page - 4095, fits into 12 bits
		static_assert(MaxBitsForFreeBins <= 16, "NextFreeBlockIndex should fit in 16 bits");

		uint64 NumFreeBins : 16;		// Number of consecutive free bins here, at least 1
		uint64 NextFreeBlock : 48;		// Next free block in another pool
	};

	struct FPoolHeader : public FFreeBlock
	{
		inline FPoolHeader(uint32 InPageSize, uint16 InBinSize, uint8 InPoolIndex, EBlockCanary InCanary)
			: FFreeBlock(InPageSize, InBinSize)
			, BinSize(InBinSize)
			, PoolIndex(InPoolIndex)
			, CanaryAndForkState(InCanary)
		{
			check(InPoolIndex < MAX_uint8 && InBinSize <= MAX_uint16);
		}

		uint16 BinSize;				// Size of the bins that this list points to
		uint8 PoolIndex;			// Index of this pool

		// Normally this value just functions as a canary to detect invalid memory state.
		// When process forking is supported, it's still a canary but it has two valid values.
		// One value is used pre-fork and one post-fork and the value is used to avoid freeing memory in pages shared with the parent process.
		EBlockCanary CanaryAndForkState;
	};

	struct FPoolList
	{
		FPoolList() = default;

		void Clear();
		bool IsEmpty() const;

		      FPoolInfo& GetFrontPool();
		const FPoolInfo& GetFrontPool() const;

		void LinkToFront(FPoolInfo* Pool);

		FPoolInfo& PushNewPoolToFront(FMallocBinned2& Allocator, FPoolTable& Table, uint32 InPoolIndex);

		void ValidateActivePools() const;
		void ValidateExhaustedPools() const;

	private:
		FPoolInfo* Front = nullptr;
	};

public:
	/** Pool table. */
	struct FPoolTable
	{
		FPoolList ActivePools;
		FPoolList ExhaustedPools;
		uint32 BinSize = 0;

#if UE_MB2_ALLOCATOR_STATS
		uint32 TotalUsedBins = 0;			// Used to calculated load factor, i.e.: 
		uint32 TotalAllocatedBins = 0;		// used bins divided by total bins number in all allocated blocks
		uint32 TotalAllocatedMem = 0;
#endif

		UE::FPlatformRecursiveMutex Mutex;

		FPoolTable() = default;
	};

	// Pool tables for different pool sizes
	FPoolTable SmallPoolTables[UE_MB2_SMALL_POOL_COUNT];

private:
#if BINNED2_FORK_SUPPORT
	EBlockCanary CurrentCanary = EBlockCanary::PreFork; // The value of the canary for pages we have allocated this side of the fork 
	EBlockCanary OldCanary = EBlockCanary::PreFork;		// If we have forked, the value canary of old pages we should avoid touching 
#else 
	static constexpr EBlockCanary CurrentCanary = EBlockCanary::Value;
#endif

#if !PLATFORM_UNIX && !PLATFORM_ANDROID
#	if UE_USE_VERYLARGEPAGEALLOCATOR
		FCachedOSVeryLargePageAllocator CachedOSPageAllocator;
#	else
		TCachedOSPageAllocator<UE_MB2_MAX_CACHED_OS_FREES, UE_MB2_MAX_CACHED_OS_FREES_BYTE_LIMIT> CachedOSPageAllocator;
#	endif
#else
	FPooledVirtualMemoryAllocator CachedOSPageAllocator;
#endif

	FORCEINLINE bool IsOSAllocation(const void* Ptr) const
	{
#if UE_USE_VERYLARGEPAGEALLOCATOR && !PLATFORM_UNIX && !PLATFORM_ANDROID
		return !CachedOSPageAllocator.IsSmallBlockAllocation(Ptr) && IsAligned(Ptr, UE_MB2_LARGE_ALLOC);
#else
		return IsAligned(Ptr, UE_MB2_LARGE_ALLOC);
#endif
	}

	static FORCEINLINE FPoolHeader* GetPoolHeaderFromPointer(void* Ptr)
	{
		return (FPoolHeader*)AlignDown(Ptr, UE_MB2_LARGE_ALLOC);
	}

#if UE_MB2_DOUBLE_FREE_DETECTION
	static void SetAndValidateDoubleFreeCanary(void* Ptr);
	static void ClearDoubleFreeCanary(void* Ptr);
	static void ReportSmallDoubleFree(void* Ptr);
#endif

public:
	FMallocBinned2();
	virtual ~FMallocBinned2();

	// FMalloc interface.
	virtual bool IsInternallyThreadSafe() const override;

	UE_AUTORTFM_NOAUTORTFM
	virtual void* Malloc(SIZE_T Size, uint32 Alignment) override;

	void* MallocNoTLS(SIZE_T Size, uint32 Alignment);

	UE_AUTORTFM_NOAUTORTFM
	virtual void* Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override;

	UE_AUTORTFM_NOAUTORTFM
	virtual void Free(void* Ptr) override;

	void FreeNoTLS(void* Ptr);

	inline bool GetSmallAllocationSize(void* Ptr, SIZE_T& SizeOut) const
	{
		if (!IsOSAllocation(Ptr))
		{
			const FPoolHeader* Header = GetPoolHeaderFromPointer(Ptr);
			CanaryTest(Header);
			SizeOut = Header->BinSize;
			return true;
		}
		return false;
	}

	inline virtual bool GetAllocationSize(void *Ptr, SIZE_T &SizeOut) override
	{
		if (GetSmallAllocationSize(Ptr, SizeOut))
		{
			return true;
		}
		return GetAllocationSizeExternal(Ptr, SizeOut);
	}

	FORCEINLINE virtual SIZE_T QuantizeSize(SIZE_T Count, uint32 Alignment) override
	{
		return QuantizeSizeCommon(Count, Alignment, *this);
	}

	virtual bool ValidateHeap() override;
	virtual void Trim(bool bTrimThreadCaches) override;
	virtual const TCHAR* GetDescriptiveName() override;
	virtual void UpdateStats() override;
	virtual void OnMallocInitialized() override;
	virtual void OnPreFork() override;
	virtual void OnPostFork() override;
	virtual uint64 GetImmediatelyFreeableCachedMemorySize() const override
	{
		return CachedOSPageAllocator.GetCachedImmediatelyFreeable();
	}
	virtual uint64 GetTotalFreeCachedMemorySize() const override
	{
		return CachedOSPageAllocator.GetCachedFreeTotal();
	}
	// End FMalloc interface.

	void* MallocExternalSmall(SIZE_T Size, uint32 Alignment);
	void* MallocExternalLarge(SIZE_T Size, uint32 Alignment);

	void CanaryFail(const FPoolHeader* Header) const;
	inline void CanaryTest(const FPoolHeader* Header) const
	{
#if BINNED2_FORK_SUPPORT
		// When we support forking there are two valid canary values.
		if (Header->CanaryAndForkState != CurrentCanary && Header->CanaryAndForkState != OldCanary)
#else
		if (Header->CanaryAndForkState != CurrentCanary)
#endif
		{
			CanaryFail(Header);
		}
	}

	/** Dumps current allocator stats to the log. */
	virtual void DumpAllocatorStats(class FOutputDevice& Ar) override;
	
	static uint16 SmallBinSizes[UE_MB2_SMALL_POOL_COUNT];
	static FMallocBinned2* MallocBinned2;
	// Mapping of sizes to small table indices
	static uint8 MemSizeToPoolIndex[1 + (UE_MB2_MAX_SMALL_POOL_SIZE >> UE_MBC_BIN_SIZE_SHIFT)];

	static void* AllocateMetaDataMemory(SIZE_T Size);
	static void FreeMetaDataMemory(void* Ptr, SIZE_T Size);

	FORCEINLINE uint32 PoolIndexToBinSize(uint32 PoolIndex) const
	{
		return SmallBinSizes[PoolIndex];
	}

	void FreeBundles(FBundleNode* Bundles, uint32 PoolIndex);

	void FlushCurrentThreadCacheInternal(bool bNewEpochOnly = false);
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#	include "HAL/CriticalSection.h"
#endif
