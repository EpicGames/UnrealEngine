// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if PLATFORM_HAS_FPlatformVirtualMemoryBlock
#include "AutoRTFM.h"
#include "HAL/Allocators/CachedOSPageAllocator.h"
#include "HAL/Allocators/PooledVirtualMemoryAllocator.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/MallocBinnedCommon.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformMutex.h"
#include "HAL/UnrealMemory.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/Atomic.h"


#ifndef UE_MB3_USE_CACHED_PAGE_ALLOCATOR_FOR_LARGE_ALLOCS
#	if PLATFORM_ANDROID || PLATFORM_IOS || PLATFORM_WINDOWS
#		define UE_MB3_USE_CACHED_PAGE_ALLOCATOR_FOR_LARGE_ALLOCS (1)
#	else
#		define UE_MB3_USE_CACHED_PAGE_ALLOCATOR_FOR_LARGE_ALLOCS (0)
#	endif
#endif

#if PLATFORM_IOS
#	define UE_MB3_MAX_MEMORY_PER_POOL_SIZE_MB		512
#elif !defined(UE_MB3_MAX_MEMORY_PER_POOL_SIZE_MB)
#	define UE_MB3_MAX_MEMORY_PER_POOL_SIZE_MB		1024
#endif

#if defined(USE_512MB_MAX_MEMORY_PER_BLOCK_SIZE)
#	warning "USE_512MB_MAX_MEMORY_PER_BLOCK_SIZE is deprecated. Please use UE_MB3_MAX_MEMORY_PER_POOL_SIZE_MB=%MB% instead"
#endif

#define UE_MB3_BASE_PAGE_SIZE						4096			// Minimum "page size" for binned3


#ifndef UE_MB3_MAX_SMALL_POOL_SIZE
#	if UE_MB3_USE_CACHED_PAGE_ALLOCATOR_FOR_LARGE_ALLOCS
#		define UE_MB3_MAX_SMALL_POOL_SIZE			(UE_MBC_MAX_LISTED_SMALL_POOL_SIZE)	// Maximum small bin size
#	else
#		define UE_MB3_MAX_SMALL_POOL_SIZE			(128 * 1024)	// Maximum small bin size
#	endif
#endif
#define UE_MB3_SMALL_POOL_COUNT						(UE_MBC_NUM_LISTED_SMALL_POOLS + (UE_MB3_MAX_SMALL_POOL_SIZE - UE_MBC_MAX_LISTED_SMALL_POOL_SIZE) / UE_MB3_BASE_PAGE_SIZE)

#define UE_MB3_MAX_MEMORY_PER_POOL_SIZE_SHIFT		(FMath::CountTrailingZeros(FMath::RoundUpToPowerOfTwo(UE_MB3_MAX_MEMORY_PER_POOL_SIZE_MB * 1024 * 1024)))

#define UE_MB3_MAX_MEMORY_PER_POOL_SIZE				(1ull << UE_MB3_MAX_MEMORY_PER_POOL_SIZE_SHIFT) 

#define UE_MB3_ALLOCATOR_STATS						UE_MBC_ALLOCATOR_STATS

#if UE_MB3_ALLOCATOR_STATS
#	define UE_M3_ALLOCATOR_PER_BIN_STATS			!(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#else
#	define UE_M3_ALLOCATOR_PER_BIN_STATS			0
#endif


PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

//
// Optimized virtual memory allocator.
// 
// MallocBinned3 supports two types of allocations - large and small pool allocation:
// 1. For small pool allocation MallocBinned3 reserves contiguous range of virtual memory (a Pool) for each allocation size (a Bin).
//    By default each Pool reserves 1 GB of address space, unless USE_512MB_MAX_MEMORY_PER_BLOCK_SIZE is defined to 1.
//    Each Pool commits and decommits reserved memory in Blocks. Each Block is equal to at least one memory page in it's size.
//    A Block contains N Bins in a way to minimize the tail memory waste. For example 16 bytes bins fit inside 4KB memory page without any waste;
//    i.e. one Block for 16 byte Bins contains 256 bins. However 736 bytes Bin uses two 4KB pages in it's Block to minimize memory waste.
//    Each Pool manages it's Blocks allocation via a Bit Tree (BlocksAllocatedBits\BlocksExhaustedBits members).
//    And every Block manages it's Bins via a number of additional data structures - FPoolInfoSmall and FFreeBlock.
//    FFreeBlock is an in-place header for the Block that's stored at the end of each block and contains info on number of free Bins and index of the next free Block if any
//    Memory is allocated top down in a Block
// 
// 2. For large allocations we go directly to OS, unless UE_MB3_USE_CACHED_PAGE_ALLOCATOR_FOR_LARGE_ALLOCS is defined to 1
//    Each allocation is handled via FPlatformVirtualMemoryBlock and information about it is stored in FPoolInfo
//    FPoolInfo stores the original allocation size and how much memory was committed by the OS in case we need to realloc that allocation and there's enough tail waste to do that in place
//    All FPoolInfo are stored in the PoolHashBucket and HashBucketFreeList
//
class FMallocBinned3 : public TMallocBinnedCommon<FMallocBinned3, UE_MB3_SMALL_POOL_COUNT, UE_MB3_MAX_SMALL_POOL_SIZE>
{
public:
	struct FPoolInfo
	{
		enum class ECanary : uint32
		{
			Unassigned = 0x39431234,
			Assigned = 0x17ea5678,
		};

		FPoolInfo();

		void CheckCanary(ECanary ShouldBe) const;
		void SetCanary(ECanary ShouldBe, bool bPreexisting, bool bGuarnteedToBeNew);

		uint32 GetOSRequestedBytes() const;
		uint32 GetOsCommittedBytes() const;

		// And alias for GetOsCommittedBytes() to align with MB2 naming convention
		uint32 GetOsAllocatedBytes() const
		{
			return GetOsCommittedBytes();
		}

		uint32 GetOsVMPages() const;
		void SetOSAllocationSize(uint32 InRequestedBytes);
		void SetOSAllocationSizes(uint32 InRequestedBytes, UPTRINT InCommittedBytes, uint32 InVMSizeDivVirtualSizeAlignment);

		ECanary	Canary;
	private:
		uint32 AllocSize;						// Number of bytes allocated
		uint32 VMSizeDivVirtualSizeAlignment;	// Number of VM bytes allocated aligned for OS
		uint32 CommitSize;						// Number of bytes committed by the OS
	};

	// Forward declarations
	struct FPoolInfoSmall;
	struct FPoolTable;
	struct Private;

	/** Pool table. */
	struct FPoolTable
	{
		uint32 BinSize;						// Bin size, i.e. 16, 32, 64 etc bytes. Pretty redundant and can be computed
		uint32 BlockSize;					// Size of a block. A block contains N bins in a way to minimize tail memory waste
		uint32 NumMemoryPagesPerBlock;		// Num memory pages needed to allocate one block.
		uint32 NumEverUsedBlocks = 0;

		FBitTree BlocksAllocatedBits;		// A bit per per block, one indicates a committed block
		FBitTree BlocksExhaustedBits;		// One here means that the block is completely full

		FPoolInfoSmall** PoolInfos;			// Book keeping info about every allocated area for this bin's pool

		uint64 UnusedAreaOffsetLow = 0;		// High watermark for allocated vm memory for this pool

#if UE_MB3_ALLOCATOR_STATS
		uint32 TotalUsedBins = 0;			// Used to calculated load factor, i.e.: 
		uint32 TotalAllocatedBins = 0;		// used bins divided by total bins number in all allocated blocks
		uint32 TotalAllocatedMem = 0;
#endif

		UE::FPlatformRecursiveMutex Mutex;

#if UE_M3_ALLOCATOR_PER_BIN_STATS
		// these are "head end" stats, above the TLS cache
		std::atomic<int64> TotalRequestedAllocSize = 0;
		std::atomic<int64> TotalAllocCount = 0;
		std::atomic<int64> TotalFreeCount = 0;

		inline void HeadEndAlloc(SIZE_T Size)
		{
			check(Size >= 0 && Size <= BinSize);
			TotalRequestedAllocSize.fetch_add(Size, std::memory_order_relaxed);
			TotalAllocCount.fetch_add(1, std::memory_order_relaxed);
		}
		FORCEINLINE void HeadEndFree()
		{
			TotalFreeCount.fetch_add(1, std::memory_order_relaxed);
		}
#else
		FORCEINLINE void HeadEndAlloc(SIZE_T Size)
		{
		}
		FORCEINLINE void HeadEndFree()
		{
		}
#endif
	};

	// Pool tables for different pool sizes
	FPoolTable SmallPoolTables[UE_MB3_SMALL_POOL_COUNT];

private:
	uint32 SmallPoolInfosPerPlatformPage;

	FORCEINLINE uint64 PoolIndexFromPtr(const void* Ptr) const		// returns a uint64 for it can also be used to check if it is an OS allocation
	{
		return (UPTRINT(Ptr) - UPTRINT(Binned3BaseVMPtr)) >> UE_MB3_MAX_MEMORY_PER_POOL_SIZE_SHIFT;
	}

	FORCEINLINE uint8* PoolBasePtr(uint32 InPoolIndex) const
	{
		return Binned3BaseVMPtr + InPoolIndex * UE_MB3_MAX_MEMORY_PER_POOL_SIZE;
	}

	inline uint32 PoolIndexFromPtrChecked(const void* Ptr) const
	{
		const uint64 Result = PoolIndexFromPtr(Ptr);
		check(Result < UE_MB3_SMALL_POOL_COUNT);
		return (uint32)Result;
	}

	FORCEINLINE bool IsOSAllocation(const void* Ptr) const
	{
		return PoolIndexFromPtr(Ptr) >= UE_MB3_SMALL_POOL_COUNT;
	}

	inline void* BlockPointerFromContainedPtr(const void* Ptr, uint8 NumMemoryPagesPerBlock, uint32& OutBlockIndex) const
	{
		const uint32 PoolIndex = PoolIndexFromPtrChecked(Ptr);
		uint8* PoolStart = PoolBasePtr(PoolIndex);
		const uint64 BlockIndex = (UPTRINT(Ptr) - UPTRINT(PoolStart)) / (UPTRINT(NumMemoryPagesPerBlock) * UPTRINT(OsAllocationGranularity));
		OutBlockIndex = BlockIndex;

		uint8* Result = PoolStart + BlockIndex * UPTRINT(NumMemoryPagesPerBlock) * UPTRINT(OsAllocationGranularity);

		check(Result < PoolStart + UE_MB3_MAX_MEMORY_PER_POOL_SIZE);
		return Result;
	}

	inline uint8* BlockPointerFromIndecies(uint32 InPoolIndex, uint32 BlockIndex, uint32 BlockSize) const
	{
		uint8* PoolStart = PoolBasePtr(InPoolIndex);
		uint8* Ptr = PoolStart + BlockIndex * uint64(BlockSize);
		check(Ptr + BlockSize <= PoolStart + UE_MB3_MAX_MEMORY_PER_POOL_SIZE);
		return Ptr;
	}

	FPoolInfoSmall* PushNewPoolToFront(FPoolTable& Table, uint32 InBinSize, uint32 InPoolIndex, uint32& OutBlockIndex);
	FPoolInfoSmall* GetFrontPool(FPoolTable& Table, uint32 InPoolIndex, uint32& OutBlockIndex);

public:

	FMallocBinned3(const FPlatformMemory::FPlatformVirtualMemoryBlock* ExternalMemoryBlock = nullptr);

	virtual ~FMallocBinned3();

	// FMalloc interface.
	virtual bool IsInternallyThreadSafe() const override;

	UE_AUTORTFM_NOAUTORTFM
	virtual void* Malloc(SIZE_T Size, uint32 Alignment) override;

	void* MallocNoTLS(SIZE_T Size, uint32 Alignment);

	UE_AUTORTFM_NOAUTORTFM
	virtual void* Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override;

	UE_AUTORTFM_NOAUTORTFM
	FORCEINLINE virtual void Free(void* Ptr) override
	{
		FreeExternal(Ptr, PoolIndexFromPtr(Ptr));
	}

	void FreeNoTLS(void* Ptr);

	inline bool GetSmallAllocationSize(void* Ptr, SIZE_T& SizeOut) const
	{
		const uint64 PoolIndex = PoolIndexFromPtr(Ptr);
		if (PoolIndex < UE_MB3_SMALL_POOL_COUNT)
		{
			SizeOut = PoolIndexToBinSize(PoolIndex);
			return true;
		}

		return false;
	}

	inline virtual bool GetAllocationSize(void *Ptr, SIZE_T &SizeOut) override
	{
		if (GetSmallAllocationSize(Ptr, SizeOut))
		{
			return true;
		}
		return GetAllocationSizeExternal(Ptr, SizeOut);
	}

	FORCEINLINE virtual SIZE_T QuantizeSize(SIZE_T Count, uint32 Alignment) override
	{
		return QuantizeSizeCommon(Count, Alignment, *this);
	}

	virtual bool ValidateHeap() override;
	virtual void Trim(bool bTrimThreadCaches) override;
	virtual const TCHAR* GetDescriptiveName() override;
	/** Dumps current allocator stats to the log. */
	virtual void DumpAllocatorStats(class FOutputDevice& Ar) override;
	virtual void UpdateStats() override;
	// End FMalloc interface.

	void FreeExternal(void* Ptr, uint64 PoolIndex);
	
	// +1 enables PoolIndexToBinSize(~0u / -1) dummy access that helps avoid PoolIndex == 0 branching in Realloc(),
	// see ((!PoolIndex) | (NewSize > PoolIndexToBinSize(static_cast<uint32>(PoolIndex - 1)))
	static uint16 SmallBinSizesShifted[UE_MB3_SMALL_POOL_COUNT + 1];
	static FMallocBinned3* MallocBinned3;

	static uint8* Binned3BaseVMPtr;
	FPlatformMemory::FPlatformVirtualMemoryBlock Binned3BaseVMBlock;

	// Mapping of sizes to small table indices
	static uint8 MemSizeToPoolIndex[SIZE_TO_POOL_INDEX_NUM];

	FORCEINLINE uint32 PoolIndexToBinSize(uint32 PoolIndex) const
	{
		return uint32(SmallBinSizesShifted[PoolIndex + 1]) << UE_MBC_BIN_SIZE_SHIFT;
	}

	void FreeBundles(FBundleNode* Bundles, uint32 PoolIndex);

	void Commit(uint32 InPoolIndex, void *Ptr, SIZE_T Size);
	void Decommit(uint32 InPoolIndex, void *Ptr, SIZE_T Size);

	void FlushCurrentThreadCacheInternal(bool bNewEpochOnly = false);

	void EnableLargeAllocsCache();
	void DisableLargeAllocsCache();

	static void* AllocateMetaDataMemory(SIZE_T Size);
	static void FreeMetaDataMemory(void *Ptr, SIZE_T Size);
};

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#	include "HAL/CriticalSection.h"
#endif

#endif	//~PLATFORM_HAS_FPlatformVirtualMemoryBlock
