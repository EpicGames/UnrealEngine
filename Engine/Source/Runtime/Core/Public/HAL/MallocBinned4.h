// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Async/UniqueLock.h"
#include "AutoRTFM.h"
#include "HAL/Allocators/CachedOSPageAllocator.h"
#include "HAL/Allocators/CachedOSVeryLargePageAllocator.h"
#include "HAL/Allocators/PooledVirtualMemoryAllocator.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/MallocBinnedCommon.h"
#include "HAL/PlatformMutex.h"
#include "HAL/UnrealMemory.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Fork.h"
#include "Templates/Atomic.h"


#define UE_MB4_MAX_CACHED_OS_FREES (64)
#define UE_MB4_MAX_CACHED_OS_FREES_BYTE_LIMIT (64*1024*1024)

#define UE_MB4_LARGE_ALLOC					65536		// Alignment of OS-allocated pointer - pool-allocated pointers will have a non-aligned pointer

#if AGGRESSIVE_MEMORY_SAVING
#	define UE_MB4_MAX_SMALL_POOL_SIZE		(13104)		// Maximum bin size in SmallBinSizesInternal in cpp file
#	define UE_MB4_SMALL_POOL_COUNT			49
#else
#	define UE_MB4_MAX_SMALL_POOL_SIZE		(32768)		// Maximum bin size in SmallBinSizesInternal in cpp file
#	define UE_MB4_SMALL_POOL_COUNT			52
#endif

// If we are emulating forking on a windows server or are a linux server, enable support for avoiding dirtying pages owned by the parent. 
#ifndef BINNED4_FORK_SUPPORT
#	define BINNED4_FORK_SUPPORT (UE_SERVER && (PLATFORM_UNIX || DEFAULT_SERVER_FAKE_FORKS))
#endif

#define UE_MB4_ALLOCATOR_STATS				UE_MBC_ALLOCATOR_STATS


//
// Optimized virtual memory allocator.
//
class FMallocBinned4 : public TMallocBinnedCommon<FMallocBinned4, UE_MB4_SMALL_POOL_COUNT, UE_MB4_MAX_SMALL_POOL_SIZE>
{
	struct FFreeBlock;

public:
	struct FPoolTable;

	// Canary value used in FFreeBlock
	// A constant value unless we're compiled with fork support in which case there are two values identifying whether the page
	// was allocated pre- or post-fork
	enum class EBlockCanary : uint8
	{
		Zero = 0x0, // Not clear why this is needed by FreeBundles
	#if BINNED4_FORK_SUPPORT
		PreFork = 0xb7,
		PostFork = 0xca,
	#else
		Value = 0xe3 
	#endif
	};

	struct FPoolInfo
	{
		enum class ECanary : uint16
		{
			Unassigned = 0x3941,
			FirstFreeBlockIsOSAllocSize = 0x17ea,
			FirstFreeBlockIsPtr = 0xf317
		};

		uint16      Taken;          // Number of allocated elements in this pool, when counts down to zero can free the entire pool	
		ECanary     Canary;         // See ECanary
		uint32      AllocSize;      // Number of bytes allocated
		FFreeBlock* FirstFreeBlock; // Pointer to first free memory in this pool or the OS Allocation Size in bytes if this allocation is not binned
		FPoolInfo*  Next;           // Pointer to next pool
		FPoolInfo** PtrToPrevNext;  // Pointer to whichever pointer points to this pool

		FPoolInfo();

		void CheckCanary(ECanary ShouldBe) const;
		void SetCanary(ECanary ShouldBe, bool bPreexisting, bool bGuaranteedToBeNew);

		bool HasFreeBin() const;
		void* AllocateBin();

		SIZE_T GetOSRequestedBytes() const;
		SIZE_T GetOsAllocatedBytes() const;
		void SetOSAllocationSizes(SIZE_T InRequestedBytes, UPTRINT InAllocatedBytes);

		void Link(FPoolInfo*& PrevNext);
		void Unlink();

	private:
		void ExhaustPoolIfNecessary();
	};

private:
	// Forward declares.
	struct Private;

	/** Information about a piece of free memory. */
	struct FFreeBlock
	{
		inline FFreeBlock(uint32 InPageSize, uint16 InBinSize, EBlockCanary InCanary)
			: BinSize(InBinSize)
			, CanaryAndForkState(InCanary)
			, NextFreeBlockIndex(InvalidNextFreeBlock)
		{
			NumFreeBins = uint16(InPageSize / InBinSize);
		}

		FORCEINLINE uint32 GetNumFreeBins() const
		{
			return NumFreeBins;
		}

		inline void* AllocateBin()
		{
			--NumFreeBins;
			// allocate high-to-low: offset 0 (the header location) is returned last
			return (uint8*)this + NumFreeBins * BinSize;
		}

		FORCEINLINE FFreeBlock* GetNextFreeBlock() const
		{
			if (NextFreeBlockIndex == InvalidNextFreeBlock)
			{
				return nullptr;
			}
			uint8* BlockBase = (uint8*)AlignDown(this, UE_MB4_LARGE_ALLOC);
			return (FFreeBlock*)(BlockBase + NextFreeBlockIndex * UE_MBC_MIN_BIN_SIZE);
		}

		FORCEINLINE void SetNextFreeBlock(FFreeBlock* Next)
		{
			if (Next)
			{
				uint8* BlockBase = (uint8*)AlignDown(this, UE_MB4_LARGE_ALLOC);
				NextFreeBlockIndex = uint16((uint8*)Next - BlockBase) / UE_MBC_MIN_BIN_SIZE;
			}
			else
			{
				NextFreeBlockIndex = InvalidNextFreeBlock;
			}
		}

		//constexpr static uint32 MaxBitsForBinSize	 = 32 - FMath::CountLeadingZeros(UE_MB4_MAX_SMALL_POOL_SIZE));
		constexpr static uint32 MaxBitsForFreeBins = 32 - FMath::CountLeadingZeros(UE_MB4_LARGE_ALLOC / UE_MBC_MIN_BIN_SIZE);
		constexpr static uint32 InvalidNextFreeBlock = 1u << MaxBitsForFreeBins;
		static_assert(MaxBitsForFreeBins + 1 <= 16, "NextFreeBlockIndex should fit in 16 bits");

		uint16 BinSize;						// Size of the bins
		EBlockCanary CanaryAndForkState;	// Canary value; two valid values when fork support is active
		uint16 NumFreeBins;					// Number of consecutive free bins here, at least 1
		uint16 NextFreeBlockIndex;			// Bin index within 64KB block of next recycled FFreeBlock, or InvalidNextFreeBlock
	};

	struct FPoolList
	{
		FPoolList() = default;

		void Clear();
		bool IsEmpty() const;

		      FPoolInfo& GetFrontPool();
		const FPoolInfo& GetFrontPool() const;

		void LinkToFront(FPoolInfo* Pool);

		FPoolInfo& PushNewPoolToFront(FMallocBinned4& Allocator, FPoolTable& Table, uint32 InPoolIndex);

		void ValidateActivePools() const;
		void ValidateExhaustedPools() const;

	private:
		FPoolInfo* Front = nullptr;
	};

public:
	/** Pool table. */
	struct FPoolTable
	{
		FPoolList ActivePools;
		FPoolList ExhaustedPools;
		uint32 BinSize = 0;

#if UE_MB4_ALLOCATOR_STATS
		uint32 TotalUsedBins = 0;			// Used to calculated load factor, i.e.: 
		uint32 TotalAllocatedBins = 0;		// used bins divided by total bins number in all allocated blocks
		uint32 TotalAllocatedMem = 0;
#endif

		UE::FPlatformRecursiveMutex Mutex;

		FPoolTable() = default;
	};

	// Pool tables for different pool sizes
	FPoolTable SmallPoolTables[UE_MB4_SMALL_POOL_COUNT];

private:
#if BINNED4_FORK_SUPPORT
	EBlockCanary CurrentCanary = EBlockCanary::PreFork; // The value of the canary for pages we have allocated this side of the fork 
	EBlockCanary OldCanary = EBlockCanary::PreFork;		// If we have forked, the value canary of old pages we should avoid touching 
#else 
	static constexpr EBlockCanary CurrentCanary = EBlockCanary::Value;
#endif

#if !PLATFORM_UNIX && !PLATFORM_ANDROID
#	if UE_USE_VERYLARGEPAGEALLOCATOR
		FCachedOSVeryLargePageAllocator CachedOSPageAllocator;
#	else
		TCachedOSPageAllocator<UE_MB4_MAX_CACHED_OS_FREES, UE_MB4_MAX_CACHED_OS_FREES_BYTE_LIMIT> CachedOSPageAllocator;
#	endif
#else
	FPooledVirtualMemoryAllocator CachedOSPageAllocator;
#endif

	// Side-table entry encoding: bits[6:0] = PoolIndex, bit 7 = pre-fork flag
	static constexpr uint8 PreForkFlag       = 0x80;
	static constexpr uint8 PoolIndexMask     = 0x7F;
	static constexpr uint8 UnregisteredBlock = 0xFF;

	uint8** BlockPoolL1Table = nullptr;

	FORCEINLINE uint8 LookupBlockPoolEntry(const void* Ptr) const
	{
		const uint64 Addr = uint64(AlignDown(Ptr, UE_MB4_LARGE_ALLOC));
		const uint8* L2 = BlockPoolL1Table[Addr >> 32];
		return L2 ? L2[(Addr >> 16) & 0xFFFF] : UnregisteredBlock;
	}

	void RegisterBlockPool(void* BlockBase, uint32 PoolIndex);
	void UnregisterBlockPool(void* BlockBase);

	// Returns the validated side-table entry for a small-pool allocation,
	// or UnregisteredBlock if Ptr is a large OS allocation (64KB-aligned and unregistered) or null.
	// Calls CanaryFail if Ptr looks like a small-pool pointer but has no registered entry.
	uint8 GetSmallBinEntry(const void* Ptr) const;

	FORCEINLINE bool IsOSAllocation(const void* Ptr) const
	{
#if UE_USE_VERYLARGEPAGEALLOCATOR && !PLATFORM_UNIX && !PLATFORM_ANDROID
		// With the very large page allocator, small pool blocks come from managed 2MB pages
		// and must be distinguished from large OS allocations via IsSmallBlockAllocation.
		// The side table provides the same distinction but we keep both checks for correctness.
		return !CachedOSPageAllocator.IsSmallBlockAllocation(Ptr) && IsAligned(Ptr, UE_MB4_LARGE_ALLOC) && LookupBlockPoolEntry(Ptr) == UnregisteredBlock;
#else
		return IsAligned(Ptr, UE_MB4_LARGE_ALLOC) && LookupBlockPoolEntry(Ptr) == UnregisteredBlock;
#endif
	}

public:
	FMallocBinned4();
	virtual ~FMallocBinned4();

	// FMalloc interface.
	virtual bool IsInternallyThreadSafe() const override;

	UE_AUTORTFM_NOAUTORTFM
	virtual void* Malloc(SIZE_T Size, uint32 Alignment) override;

	UE_AUTORTFM_NOAUTORTFM
	virtual void* Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override;

	UE_AUTORTFM_NOAUTORTFM
	virtual void Free(void* Ptr) override;

	inline bool GetSmallAllocationSize(void* Ptr, SIZE_T& SizeOut) const
	{
		const uint8 Entry = LookupBlockPoolEntry(Ptr);
		if (Entry == UnregisteredBlock) return false;
		const uint32 PoolIndex = Entry & PoolIndexMask;
		CA_ASSUME(PoolIndex < UE_MB4_SMALL_POOL_COUNT);
		SizeOut = SmallPoolTables[PoolIndex].BinSize;
		return true;
	}

	inline virtual bool GetAllocationSize(void *Ptr, SIZE_T &SizeOut) override
	{
		if (GetSmallAllocationSize(Ptr, SizeOut))
		{
			return true;
		}
		return GetAllocationSizeExternal(Ptr, SizeOut);
	}

	FORCEINLINE virtual SIZE_T QuantizeSize(SIZE_T Count, uint32 Alignment) override
	{
		return QuantizeSizeCommon(Count, Alignment, *this);
	}

	virtual bool ValidateHeap() override;
	virtual void Trim(bool bTrimThreadCaches) override;
	virtual const TCHAR* GetDescriptiveName() override;
	virtual void UpdateStats() override;
	virtual void OnMallocInitialized() override;
	virtual void OnPreFork() override;
	virtual void OnPostFork() override;
	virtual uint64 GetImmediatelyFreeableCachedMemorySize() const override
	{
		return CachedOSPageAllocator.GetCachedImmediatelyFreeable();
	}
	virtual uint64 GetTotalFreeCachedMemorySize() const override
	{
		return CachedOSPageAllocator.GetCachedFreeTotal();
	}
	// End FMalloc interface.

	void* MallocExternalSmall(SIZE_T Size, uint32 Alignment);
	void* MallocExternalLarge(SIZE_T Size, uint32 Alignment);

	void CanaryFail(const void* Ptr) const;
	inline void CanaryTest(const void* Ptr) const
	{
		// The side table is used instead of reading FFreeBlock::CanaryAndForkState because
		// the new AllocateBin direction returns offset 0 (the header location) as the last bin,
		// so the block header may be user-owned memory at the time of Free/Realloc.
		// Pre-fork blocks remain registered in the side table (with bit7 set) and therefore
		// pass this test, matching the original two-canary BINNED4_FORK_SUPPORT behavior.
		if (LookupBlockPoolEntry(Ptr) == UnregisteredBlock)
		{
			CanaryFail(Ptr);
		}
	}

	/** Dumps current allocator stats to the log. */
	virtual void DumpAllocatorStats(class FOutputDevice& Ar) override;
	
	static uint16 SmallBinSizes[UE_MB4_SMALL_POOL_COUNT];
	static FMallocBinned4* MallocBinned4;
	// Mapping of sizes to small table indices
	static uint8 MemSizeToPoolIndex[1 + (UE_MB4_MAX_SMALL_POOL_SIZE >> UE_MBC_BIN_SIZE_SHIFT)];

	static void* AllocateMetaDataMemory(SIZE_T Size);
	static void FreeMetaDataMemory(void* Ptr, SIZE_T Size);

	FORCEINLINE uint32 PoolIndexToBinSize(uint32 PoolIndex) const
	{
		return SmallBinSizes[PoolIndex];
	}

	void FreeBundles(FBundleNode* Bundles, uint32 PoolIndex);

	void FlushCurrentThreadCacheInternal(bool bNewEpochOnly = false);
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#	include "HAL/CriticalSection.h"
#endif
