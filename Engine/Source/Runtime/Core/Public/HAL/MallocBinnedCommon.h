// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include <atomic>
#include "HAL/MemoryBase.h"
#include "HAL/PlatformMutex.h"
#include "HAL/PlatformTLS.h"
#include "Async/UniqueLock.h"
#include "Async/WordMutex.h"
#include "Misc/ScopeLock.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/MemoryOps.h"

// A project can define it's own UE_MBC_MAX_LISTED_SMALL_POOL_SIZE and UE_MBC_NUM_LISTED_SMALL_POOLS to reduce runtime memory usage
// MallocBinnedCommon.cpp has a list of predefined bins that go up to 28672
// By default allocators (i.e. MB3) that use these bins will rely on this number as a baseline for a small bins count
// These allocators can increase the amount of small bins they want to manage by going over the default MBC bins list
// In MB3 case that means defining BINNED3_MAX_SMALL_POOL_SIZE to something like 65536
// Every bin over the UE_MBC_MAX_LISTED_SMALL_POOL_SIZE would come with a 4kb increment
// These small bins would be kept in user mode, increasing application's memory footprint and reducing the time it takes to allocate memory from the said bins
// If application needs to aggressively reduce it's memory footprint, potentially trading some perf due to an increased amount of kernel calls to allocate memory
// it can redefine UE_MBC_MAX_LISTED_SMALL_POOL_SIZE and BINNED3_MAX_SMALL_POOL_SIZE to smaller numbers, the good value is 16384 for both
// This, however, would require the app to redefine UE_MBC_NUM_LISTED_SMALL_POOLS too to match the number of bins that fall under the new define's threshold
// In case of 16384, we'll skip 3 larger bins and so UE_MBC_NUM_LISTED_SMALL_POOLS should be set to 48 at the time of writing
#if !defined(UE_MBC_MAX_LISTED_SMALL_POOL_SIZE)
#	define UE_MBC_MAX_LISTED_SMALL_POOL_SIZE			28672
#endif

#if !defined(UE_MBC_NUM_LISTED_SMALL_POOLS)
#	define UE_MBC_NUM_LISTED_SMALL_POOLS				52
#endif

#define UE_MBC_MIN_SMALL_POOL_ALIGNMENT					8	// Minimum alignment of bins. Added to support 8 bytes bin. If you need to mask lower bits - use this!
#define UE_MBC_MAX_SMALL_POOL_ALIGNMENT					256
#define UE_MBC_STANDARD_ALIGNMENT						16	// Standard alignment for all allocations, except 8 bytes that is aligned by 8 bytes
#define UE_MBC_MIN_BIN_SIZE								8	// Minimum supported bin size
#define UE_MBC_BIN_SIZE_SHIFT							3	// Shift for a bin size to save memory

#if !defined(AGGRESSIVE_MEMORY_SAVING)
#	error "AGGRESSIVE_MEMORY_SAVING must be defined"
#endif

#if AGGRESSIVE_MEMORY_SAVING
#	define UE_DEFAULT_GMallocBinnedBundleSize			8192
#else
#	define UE_DEFAULT_GMallocBinnedBundleSize			65536
#endif

#if !defined(UE_DEFAULT_GMallocBinnedPerThreadCaches)
#	define UE_DEFAULT_GMallocBinnedPerThreadCaches		1
#endif
#define UE_DEFAULT_GMallocBinnedBundleCount				64
#define UE_DEFAULT_GMallocBinnedAllocExtra				32
#define UE_DEFAULT_GMallocBinnedMaxBundlesBeforeRecycle	8

#ifndef UE_MBC_ALLOW_RUNTIME_TWEAKING
#	define UE_MBC_ALLOW_RUNTIME_TWEAKING				0
#elif UE_MBC_ALLOW_RUNTIME_TWEAKING
#	ifndef UE_MBC_SUPPORT_RUNTIME_TLC_TOGGLE
#		define UE_MBC_SUPPORT_RUNTIME_TLC_TOGGLE		1
#	endif
#endif

#ifndef UE_MBC_SUPPORT_RUNTIME_TLC_TOGGLE
#	define UE_MBC_SUPPORT_RUNTIME_TLC_TOGGLE			1
#endif

#if UE_MBC_SUPPORT_RUNTIME_TLC_TOGGLE
	extern CORE_API int32 GMallocBinnedPerThreadCaches;
#else
#	define GMallocBinnedPerThreadCaches					UE_DEFAULT_GMallocBinnedPerThreadCaches
#endif

#if UE_MBC_ALLOW_RUNTIME_TWEAKING
	extern CORE_API int32 GMallocBinnedBundleSize;
	extern CORE_API int32 GMallocBinnedBundleCount;
	extern CORE_API int32 GMallocBinnedAllocExtra;
	extern CORE_API int32 GMallocBinnedMaxBundlesBeforeRecycle;
#else
#	define GMallocBinnedBundleSize						UE_DEFAULT_GMallocBinnedBundleSize
#	define GMallocBinnedBundleCount						UE_DEFAULT_GMallocBinnedBundleCount
#	define GMallocBinnedAllocExtra						UE_DEFAULT_GMallocBinnedAllocExtra
#	define GMallocBinnedMaxBundlesBeforeRecycle			UE_DEFAULT_GMallocBinnedMaxBundlesBeforeRecycle
#endif	//~UE_MBC_ALLOW_RUNTIME_TWEAKING

#ifndef UE_MBC_ALLOCATOR_STATS
#	define UE_MBC_ALLOCATOR_STATS (!UE_BUILD_SHIPPING || WITH_EDITOR)
#endif

#if UE_MBC_ALLOCATOR_STATS
#	define UE_MBC_UPDATE_STATS(x) x
	extern CORE_API int32 GMallocBinnedEnableCSVStats;
#else
#	define UE_MBC_UPDATE_STATS(x)
#endif

#ifndef UE_MBC_LOG_LARGE_ALLOCATION
#	define UE_MBC_LOG_LARGE_ALLOCATION 0
#endif

#ifndef UE_MBC_LIGHTWEIGHT_BIN_CALLSTACK_TRACKER
#	define UE_MBC_LIGHTWEIGHT_BIN_CALLSTACK_TRACKER		0
#endif

#if UE_MBC_LIGHTWEIGHT_BIN_CALLSTACK_TRACKER && !UE_MBC_ALLOCATOR_STATS
#	error "MB lightweight bin callstack tracker needs UE_MBC_ALLOCATOR_STATS to be enabled."
#endif

#define UE_MBC_MAX_SUPPORTED_PLATFORM_PAGE_SIZE			16 * 1024	// MB3 and it's bins were designed around 4\16KB native page size. Having larger page size will increase memory waste and will be inefficient for current allocator design

extern CORE_API float GMallocBinnedFlushThreadCacheMaxWaitTime;
extern CORE_API int32 GMallocBinnedFlushRegisteredThreadCachesOnOneThread;

struct FGenericMemoryStats;


class FBitTree
{
	uint64* Bits; // one bits in middle layers mean "all allocated"
	uint32 Capacity; // rounded up to a power of two
	uint32 DesiredCapacity;
	uint32 Rows;
	uint32 OffsetOfLastRow;
	uint32 AllocationSize;

public:
	FBitTree()
		: Bits(nullptr)
	{
	}

	static constexpr uint32 GetMemoryRequirements(uint32 NumPages)
	{
		uint32 AllocationSize = 8;
		uint32 RowsUint64s = 1;
		uint32 Capacity = 64;
		uint32 OffsetOfLastRow = 0;

		while (Capacity < NumPages)
		{
			Capacity *= 64;
			RowsUint64s *= 64;
			OffsetOfLastRow = AllocationSize / 8;
			AllocationSize += 8 * RowsUint64s;
		}

		uint32 LastRowTotal = (AllocationSize - OffsetOfLastRow * 8) * 8;
		uint32 ExtraBits = LastRowTotal - NumPages;
		AllocationSize -= (ExtraBits / 64) * 8;
		return AllocationSize;
	}

	void FBitTreeInit(uint32 InDesiredCapacity, void * Memory, uint32 MemorySize, bool InitialValue);
	uint32 AllocBit();
	bool IsAllocated(uint32 Index) const;
	void AllocBit(uint32 Index);
	uint32 NextAllocBit() const;
	uint32 NextAllocBit(uint32 StartIndex) const;
	void FreeBit(uint32 Index);
	uint32 CountOnes(uint32 UpTo) const;

	uint32 Slow_NextAllocBits(uint32 NumBits, uint64 StartIndex); // Warning, slow! NumBits must be a power of two or a multiple of 64.
};

struct FSizeTableEntry
{
	uint32 BinSize;
	uint32 NumMemoryPagesPerBlock;

	FSizeTableEntry() = default;
	FSizeTableEntry(uint32 InBinSize, uint64 PlatformPageSize, uint8 Num4kbPages, uint32 BasePageSize);

	bool operator<(const FSizeTableEntry& Other) const
	{
		return BinSize < Other.BinSize;
	}
	static uint8 FillSizeTable(uint64 PlatformPageSize, FSizeTableEntry* SizeTable, uint32 BasePageSize, uint32 MaxSize, uint32 SizeIncrement);
};


class FMallocBinnedCommonBase : public FMalloc
{
public:
	// This needs to be small enough to fit inside the smallest allocation handled by MallocBinned2\3
	struct FBundleNode
	{
		uint64 NextNodeInCurrentBundle : 48;
		uint64 Count : 8;		// 8 bits is enough to store count as UE_DEFAULT_GMallocBinnedBundleCount is 64
		uint64 Reserved : 8;	// Reserved for ARM HW for TBI, MTE, etc

		void SetNextNodeInCurrentBundle(FBundleNode* Next)
		{
			NextNodeInCurrentBundle = reinterpret_cast<uint64>(Next);
		}

		FBundleNode* GetNextNodeInCurrentBundle()
		{
			return (FBundleNode*)NextNodeInCurrentBundle;
		}
	};

	CORE_API virtual void OnMallocInitialized() override; 

protected:
	struct FPtrToPoolMapping
	{
		FPtrToPoolMapping()
			: PtrToPoolPageBitShift(0)
			, HashKeyShift(0)
			, PoolMask(0)
			, MaxHashBuckets(0)
			, AddressSpaceBase(0)
		{
		}

		explicit FPtrToPoolMapping(uint32 InPageSize, uint64 InNumPoolsPerPage, uint64 AddressBase, uint64 AddressLimit)
		{
			Init(InPageSize, InNumPoolsPerPage, AddressBase, AddressLimit);
		}

		void Init(uint32 InPageSize, uint64 InNumPoolsPerPage, uint64 AddressBase, uint64 AddressLimit)
		{
			const uint64 PoolPageToPoolBitShift = FPlatformMath::CeilLogTwo64(InNumPoolsPerPage);

			PtrToPoolPageBitShift = FPlatformMath::CeilLogTwo(InPageSize);
			HashKeyShift = PtrToPoolPageBitShift + PoolPageToPoolBitShift;
			PoolMask = (1ull << PoolPageToPoolBitShift) - 1;
			MaxHashBuckets = FMath::RoundUpToPowerOfTwo64(AddressLimit - AddressBase) >> HashKeyShift;
			AddressSpaceBase = AddressBase;
		}

		inline void GetHashBucketAndPoolIndices(const void* InPtr, uint32& OutBucketIndex, UPTRINT& OutBucketCollision, uint32& OutPoolIndex) const
		{
			check((UPTRINT)InPtr >= AddressSpaceBase);
			const UPTRINT Ptr = (UPTRINT)InPtr - AddressSpaceBase;
			OutBucketCollision = Ptr >> HashKeyShift;
			OutBucketIndex = uint32(OutBucketCollision & (MaxHashBuckets - 1));
			OutPoolIndex = uint32((Ptr >> PtrToPoolPageBitShift) & PoolMask);
		}

		UE_FORCEINLINE_HINT uint64 GetMaxHashBuckets() const
		{
			return MaxHashBuckets;
		}

	private:
		/** Shift to apply to a pointer to get the reference from the indirect tables */
		uint64 PtrToPoolPageBitShift;

		/** Shift required to get required hash table key. */
		uint64 HashKeyShift;

		/** Used to mask off the bits that have been used to lookup the indirect table */
		uint64 PoolMask;

		// PageSize dependent constants
		uint64 MaxHashBuckets;

		// Base address for any virtual allocations. Can be non 0 on some platforms
		uint64 AddressSpaceBase;
	};

private:
	struct FBundle
	{
		UE_FORCEINLINE_HINT FBundle()
		{
			Reset();
		}

		inline void Reset()
		{
			Head = nullptr;
			Count = 0;
		}

		inline void PushHead(FBundleNode* Node)
		{
			Node->SetNextNodeInCurrentBundle(Head);
			Head = Node;
			Count++;
		}

		inline FBundleNode* PopHead()
		{
			FBundleNode* Result = Head;

			Count--;
			Head = Head->GetNextNodeInCurrentBundle();
			return Result;
		}

		FBundleNode* Head;
		uint32       Count;
	};

protected:
	struct FFreeBlockList
	{
		// return true if we actually pushed it
		inline bool PushToFront(void* InPtr, uint32 InPoolIndex, uint32 InBinSize)
		{
			checkSlow(InPtr);

			if ((PartialBundle.Count >= (uint32)GMallocBinnedBundleCount) | (PartialBundle.Count * InBinSize >= (uint32)GMallocBinnedBundleSize))
			{
				if (FullBundle.Head)
				{
					return false;
				}
				FullBundle = PartialBundle;
				PartialBundle.Reset();
			}
			PartialBundle.PushHead((FBundleNode*)InPtr);
			return true;
		}

		UE_FORCEINLINE_HINT bool CanPushToFront(uint32 InPoolIndex, uint32 InBinSize) const
		{
			return !((!!FullBundle.Head) & ((PartialBundle.Count >= (uint32)GMallocBinnedBundleCount) | (PartialBundle.Count * InBinSize >= (uint32)GMallocBinnedBundleSize)));
		}

		inline void* PopFromFront(uint32 InPoolIndex)
		{
			if ((!PartialBundle.Head) & (!!FullBundle.Head))
			{
				PartialBundle = FullBundle;
				FullBundle.Reset();
			}
			return PartialBundle.Head ? PartialBundle.PopHead() : nullptr;
		}

		// tries to recycle the full bundle, if that fails, it is returned for freeing
		template <class T>
		FBundleNode* RecyleFull(uint32 InPoolIndex, T& InGlobalRecycler)
		{
			FBundleNode* Result = nullptr;
			if (FullBundle.Head)
			{
				FullBundle.Head->Count = FullBundle.Count;
				if (!InGlobalRecycler.PushBundle(InPoolIndex, FullBundle.Head))
				{
					Result = FullBundle.Head;
				}
				FullBundle.Reset();
			}
			return Result;
		}

		template <class T>
		bool ObtainPartial(uint32 InPoolIndex, T& InGlobalRecycler)
		{
			if (!PartialBundle.Head)
			{
				PartialBundle.Count = 0;
				PartialBundle.Head = InGlobalRecycler.PopBundle(InPoolIndex);
				if (PartialBundle.Head)
				{
					PartialBundle.Count = PartialBundle.Head->Count;
					return true;
				}
				return false;
			}
			return true;
		}

		FBundleNode* PopBundles(uint32 InPoolIndex)
		{
			FBundleNode* Partial = PartialBundle.Head;
			if (Partial)
			{
				PartialBundle.Reset();
			}

			FBundleNode* Full = FullBundle.Head;
			if (Full)
			{
				FullBundle.Reset();
			}

			FBundleNode* Result = Partial;
			if (Result)
			{
				FBundleNode* Prev = Result;
				FBundleNode* Next = Result->GetNextNodeInCurrentBundle();
				while (Next)
				{
					Prev = Next;
					Next = Next->GetNextNodeInCurrentBundle();
				}
				Prev->SetNextNodeInCurrentBundle(Full);
			}
			else
			{
				Result = Full;
			}

			return Result;
		}

	private:
		FBundle PartialBundle;
		FBundle FullBundle;
	};

	FPtrToPoolMapping PtrToPoolMapping;
	uint64 NumPoolsPerPage;		// Number of AllocType::FPoolInfo
	static uint32 OsAllocationGranularity;
	UE::FPlatformRecursiveMutex ExternalAllocMutex;

	std::atomic<uint64> MemoryTrimEpoch{ 0 };

#if UE_MBC_ALLOCATOR_STATS
	static std::atomic<int64> TLSMemory;
	static std::atomic<int64> ConsolidatedMemory;
	static std::atomic<int64> AllocatedSmallPoolMemory;				// requested small pool memory allocations
	static std::atomic<int64> AllocatedOSSmallPoolMemory;			// total small pool memory allocated by the os, always larger than AllocatedSmallPoolMemory
	static std::atomic<int64> AllocatedLargePoolMemory;				// memory requests to the OS which don't fit in the small pool
	static std::atomic<int64> AllocatedLargePoolMemoryWAlignment;	// when we allocate at OS level we need to align to a size

	static int64 PoolInfoMemory;
	static int64 HashMemory;

	void GetAllocatorStatsInternal(FGenericMemoryStats& OutStats, int64 TotalAllocatedSmallPoolMemory);
#endif

	[[noreturn]] static void OutOfMemory(uint64 Size, uint32 Alignment = 0)
	{
		// this is expected not to return
		FPlatformMemory::OnOutOfMemory(Size, Alignment);
	}

	[[noreturn]] void UnrecognizedPointerFatalError(void* Ptr);

#if UE_MBC_LOG_LARGE_ALLOCATION
	void LogLargeAllocation(SIZE_T Size) const;
#else
	void LogLargeAllocation(SIZE_T Size) const {};
#endif

#if UE_MBC_LIGHTWEIGHT_BIN_CALLSTACK_TRACKER
	void TrackBinAllocation(uint32 BinSize, uint32 TotalAllocatedMem);
#endif
};

template <class AllocType, int NumSmallPools, int MaxSmallPoolSize>
class TMallocBinnedCommon : public FMallocBinnedCommonBase
{
	friend class FMallocBinnedCommonUtils;

	static constexpr int NUM_SMALL_POOLS     = NumSmallPools;
	static constexpr int MAX_SMALL_POOL_SIZE = MaxSmallPoolSize;

	/** Hash table struct for retrieving allocation book keeping information */
	struct FPoolHashBucket
	{
		UPTRINT			 BucketIndex;
		typename AllocType::FPoolInfo* FirstPool;
		FPoolHashBucket* Prev;
		FPoolHashBucket* Next;

		FPoolHashBucket()
		{
			BucketIndex = 0;
			FirstPool = nullptr;
			Prev = this;
			Next = this;
		}

		void Link(FPoolHashBucket* After)
		{
			After->Prev = Prev;
			After->Next = this;
			Prev->Next = After;
			this->Prev = After;
		}

		void Unlink()
		{
			Next->Prev = Prev;
			Prev->Next = Next;
			Prev = this;
			Next = this;
		}
	};

public:
	virtual void GetAllocatorStats(FGenericMemoryStats& OutStats) override
	{
#if UE_MBC_ALLOCATOR_STATS
		GetAllocatorStatsInternal(OutStats, GetTotalAllocatedSmallPoolMemory());
#endif
	}

protected:
	static constexpr int SIZE_TO_POOL_INDEX_NUM = 1 + (MAX_SMALL_POOL_SIZE >> UE_MBC_BIN_SIZE_SHIFT);

	struct FPerThreadFreeBlockLists
	{
		static thread_local FPerThreadFreeBlockLists* ThreadSingleton;

		inline static FPerThreadFreeBlockLists* Get() TSAN_SAFE
		{
			// If the current thread doesn't have the Lock, we can't return the TLS cache for being used on the current thread as we risk racing with another thread doing trimming.
			// This can only happen in such a scenario.
			//
			//  FMemory::MarkTLSCachesAsUnusedOnCurrentThread();
			//  Node->Event->Wait(); <----- UNSAFE to use the TLS cache by its owner thread but can happen when the wait implementation allocates or frees something.
			//  FMemory::MarkTLSCachesAsUsedOnCurrentThread();
			if (ThreadSingleton && ThreadSingleton->bLockedByOwnerThread)
			{
				return ThreadSingleton;
			}
			return nullptr;
		}

		static void SetTLS()
		{
			if (!ThreadSingleton)
			{
				const int64 TLSSize = Align(sizeof(FPerThreadFreeBlockLists), AllocType::OsAllocationGranularity);
				ThreadSingleton = new (AllocType::AllocateMetaDataMemory(TLSSize)) FPerThreadFreeBlockLists();
				UE_MBC_UPDATE_STATS(TLSMemory.fetch_add(TLSSize, std::memory_order_relaxed));

				verify(ThreadSingleton);
				ThreadSingleton->bLockedByOwnerThread = true;
				ThreadSingleton->Lock();

				RegisterThreadFreeBlockLists(ThreadSingleton);
			}
		}

		static void UnlockTLS()
		{
			if (ThreadSingleton)
			{
				ThreadSingleton->bLockedByOwnerThread = false;
				ThreadSingleton->Unlock();
			}
		}

		static void LockTLS()
		{
			if (ThreadSingleton)
			{
				ThreadSingleton->Lock();
				ThreadSingleton->bLockedByOwnerThread = true;
			}
		}

		static void ClearTLS()
		{
			if (ThreadSingleton)
			{
				const int64 TLSSize = Align(sizeof(FPerThreadFreeBlockLists), AllocType::OsAllocationGranularity);
				UE_MBC_UPDATE_STATS(TLSMemory.fetch_sub(TLSSize, std::memory_order_relaxed));

				UnregisterThreadFreeBlockLists(ThreadSingleton);
				ThreadSingleton->bLockedByOwnerThread = false;
				ThreadSingleton->Unlock();
				ThreadSingleton->~FPerThreadFreeBlockLists();

				AllocType::FreeMetaDataMemory(ThreadSingleton, TLSSize);

				ThreadSingleton = nullptr;
			}
		}

		UE_FORCEINLINE_HINT void* Malloc(uint32 InPoolIndex)
		{
			return FreeLists[InPoolIndex].PopFromFront(InPoolIndex);
		}

		// return true if the pointer was pushed
		UE_FORCEINLINE_HINT bool Free(void* InPtr, uint32 InPoolIndex, uint32 InBinSize)
		{
			return FreeLists[InPoolIndex].PushToFront(InPtr, InPoolIndex, InBinSize);
		}

		// return true if a pointer can be pushed
		UE_FORCEINLINE_HINT bool CanFree(uint32 InPoolIndex, uint32 InBinSize) const
		{
			return FreeLists[InPoolIndex].CanPushToFront(InPoolIndex, InBinSize);
		}

		// returns a bundle that needs to be freed if it can't be recycled
		template <class T>
		FBundleNode* RecycleFullBundle(uint32 InPoolIndex, T& InGlobalRecycler)
		{
			return FreeLists[InPoolIndex].RecyleFull(InPoolIndex, InGlobalRecycler);
		}

		// returns true if we have anything to pop
		template <class T>
		bool ObtainRecycledPartial(uint32 InPoolIndex, T& InGlobalRecycler)
		{
			return FreeLists[InPoolIndex].ObtainPartial(InPoolIndex, InGlobalRecycler);
		}

		FBundleNode* PopBundles(uint32 InPoolIndex)
		{
			return FreeLists[InPoolIndex].PopBundles(InPoolIndex);
		}

		void Lock()
		{
			Mutex.Lock();
		}

		bool TryLock()
		{
			return Mutex.TryLock();
		}

		void Unlock()
		{
			Mutex.Unlock();
		}

		// should only be called from inside the Lock.
		bool UpdateEpoch(uint64 NewEpoch)
		{
			if (MemoryTrimEpoch >= NewEpoch)
			{
				return false;
			}

			MemoryTrimEpoch = NewEpoch;
			return true;
		}

#if UE_MBC_ALLOCATOR_STATS
	public:
		int64 AllocatedMemory = 0;
#endif
	private:
		UE::FWordMutex Mutex;
		uint64 MemoryTrimEpoch = 0;
		FFreeBlockList FreeLists[NUM_SMALL_POOLS];
		bool bLockedByOwnerThread = false;
	};

	struct Internal
	{
		using PoolInfo = typename AllocType::FPoolInfo;
		/**
		* Gets the PoolInfo for a large block memory address. If no valid info exists one is created.
		*/
		static PoolInfo* GetOrCreatePoolInfo(AllocType& Allocator, void* InPtr, typename PoolInfo::ECanary Kind)
		{
			/**
			* Creates an array of PoolInfo structures for tracking allocations.
			*/
			auto CreatePoolArray = [&Allocator](uint64 NumPools)
				{
					const uint64 PoolArraySize = NumPools * sizeof(PoolInfo);

					void* Result = Allocator.AllocateMetaDataMemory(PoolArraySize);
					UE_MBC_UPDATE_STATS(PoolInfoMemory += PoolArraySize);

					if (!Result)
					{
						Allocator.ExternalAllocMutex.Unlock();
						OutOfMemory(PoolArraySize);  // OutOfMemory is fatal and does not return
					}

					DefaultConstructItems<PoolInfo>(Result, NumPools);
					return (PoolInfo*)Result;
				};

			uint32  BucketIndex;
			UPTRINT BucketIndexCollision;
			uint32  PoolIndex;
			Allocator.PtrToPoolMapping.GetHashBucketAndPoolIndices(InPtr, BucketIndex, BucketIndexCollision, PoolIndex);

			FPoolHashBucket* FirstBucket = &Allocator.HashBuckets[BucketIndex];
			FPoolHashBucket* Collision = FirstBucket;
			do
			{
				if (!Collision->FirstPool)
				{
					Collision->BucketIndex = BucketIndexCollision;
					Collision->FirstPool = CreatePoolArray(Allocator.NumPoolsPerPage);
					Collision->FirstPool[PoolIndex].SetCanary(Kind, false, true);
					return &Collision->FirstPool[PoolIndex];
				}

				if (Collision->BucketIndex == BucketIndexCollision)
				{
					Collision->FirstPool[PoolIndex].SetCanary(Kind, false, false);
					return &Collision->FirstPool[PoolIndex];
				}

				Collision = Collision->Next;
			} while (Collision != FirstBucket);

			// Create a new hash bucket entry
			if (!Allocator.HashBucketFreeList)
			{
				Allocator.HashBucketFreeList = (FPoolHashBucket*)Allocator.AllocateMetaDataMemory(AllocType::OsAllocationGranularity);
				UE_MBC_UPDATE_STATS(HashMemory += AllocType::OsAllocationGranularity);

				for (UPTRINT i = 0, n = AllocType::OsAllocationGranularity / sizeof(FPoolHashBucket); i < n; ++i)
				{
					Allocator.HashBucketFreeList->Link(new (Allocator.HashBucketFreeList + i) FPoolHashBucket());
				}
			}

			FPoolHashBucket* NextFree = Allocator.HashBucketFreeList->Next;
			FPoolHashBucket* NewBucket = Allocator.HashBucketFreeList;

			NewBucket->Unlink();

			if (NextFree == NewBucket)
			{
				NextFree = nullptr;
			}
			Allocator.HashBucketFreeList = NextFree;

			if (!NewBucket->FirstPool)
			{
				NewBucket->FirstPool = CreatePoolArray(Allocator.NumPoolsPerPage);
				NewBucket->FirstPool[PoolIndex].SetCanary(Kind, false, true);
			}
			else
			{
				NewBucket->FirstPool[PoolIndex].SetCanary(Kind, false, false);
			}

			NewBucket->BucketIndex = BucketIndexCollision;

			FirstBucket->Link(NewBucket);

			return &NewBucket->FirstPool[PoolIndex];
		}

		static PoolInfo* FindPoolInfo(AllocType& Allocator, void* InPtr)
		{
			uint32  BucketIndex;
			UPTRINT BucketIndexCollision;
			uint32  PoolIndex;
			Allocator.PtrToPoolMapping.GetHashBucketAndPoolIndices(InPtr, BucketIndex, BucketIndexCollision, PoolIndex);

			FPoolHashBucket* FirstBucket = &Allocator.HashBuckets[BucketIndex];
			FPoolHashBucket* Collision = FirstBucket;
			do
			{
				if (Collision->BucketIndex == BucketIndexCollision)
				{
					return &Collision->FirstPool[PoolIndex];
				}

				Collision = Collision->Next;
			} while (Collision != FirstBucket);

			return nullptr;
		}
	};

	static UE::FPlatformRecursiveMutex& GetFreeBlockListsRegistrationMutex()
	{
		static UE::FPlatformRecursiveMutex FreeBlockListsRegistrationMutex;
		return FreeBlockListsRegistrationMutex;
	}

	static TArray<FPerThreadFreeBlockLists*>& GetRegisteredFreeBlockLists()
	{
		static TArray<FPerThreadFreeBlockLists*> RegisteredFreeBlockLists;
		return RegisteredFreeBlockLists;
	}

	virtual void SetupTLSCachesOnCurrentThread() override
	{
		//NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned_SetupTLSCachesOnCurrentThread);

		if (!GMallocBinnedPerThreadCaches)
		{
			return;
		}

		FPerThreadFreeBlockLists::SetTLS();
	}

	virtual void ClearAndDisableTLSCachesOnCurrentThread() override
	{
		//NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned_ClearTLSCachesOnCurrentThread);

		if (UE_MBC_SUPPORT_RUNTIME_TLC_TOGGLE || GMallocBinnedPerThreadCaches)
		{
			// we need to flush cache even if it's disabled in case of run time tweaking disabled it, but it wasn't cleaned up
			((AllocType*)this)->FlushCurrentThreadCacheInternal();
			FPerThreadFreeBlockLists::ClearTLS();
		}
	}

	virtual void MarkTLSCachesAsUsedOnCurrentThread() override
	{
		//NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned_MarkTLSCachesAsUsedOnCurrentThread);

		if (UE_MBC_SUPPORT_RUNTIME_TLC_TOGGLE || GMallocBinnedPerThreadCaches)
		{
			// when UE_MBC_SUPPORT_RUNTIME_TLC_TOGGLE is set to 1 we still need to lock cache even if it is disabled, so we will clean it up properly
			FPerThreadFreeBlockLists::LockTLS();
		}
	}

	virtual void MarkTLSCachesAsUnusedOnCurrentThread() override
	{
		//NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned_MarkTLSCachesAsUnusedOnCurrentThread);

		if (UE_MBC_SUPPORT_RUNTIME_TLC_TOGGLE || GMallocBinnedPerThreadCaches)
		{
			// Will only flush if memory trimming epoch has been bumped while the thread was active.
			const bool bNewEpochOnly = true;
			((AllocType*)this)->FlushCurrentThreadCacheInternal(bNewEpochOnly);

			FPerThreadFreeBlockLists::UnlockTLS();
		}
	}

	inline SIZE_T QuantizeSizeCommon(SIZE_T Count, uint32 Alignment, const AllocType& Alloc) const
	{
		checkSlow(FMath::IsPowerOfTwo(Alignment));
		SIZE_T SizeOut;
		if ((Count <= MAX_SMALL_POOL_SIZE) & (Alignment <= UE_MBC_STANDARD_ALIGNMENT)) // one branch, not two
		{
			SizeOut = Alloc.PoolIndexToBinSize(BoundSizeToPoolIndex(Count, Alloc.MemSizeToPoolIndex));
			check(SizeOut >= Count);
			return SizeOut;
		}
		Alignment = FMath::Max<uint32>(Alignment, UE_MBC_STANDARD_ALIGNMENT);
		Count = Align(Count, Alignment);
		if ((Count <= MAX_SMALL_POOL_SIZE) & (Alignment <= UE_MBC_MAX_SMALL_POOL_ALIGNMENT))
		{
			uint32 PoolIndex = BoundSizeToPoolIndex(Count, Alloc.MemSizeToPoolIndex);
			do
			{
				const uint32 BinSize = Alloc.PoolIndexToBinSize(PoolIndex);
				if (IsAligned(BinSize, Alignment))
				{
					SizeOut = SIZE_T(BinSize);
					check(SizeOut >= Count);
					return SizeOut;
				}

				PoolIndex++;
			} while (PoolIndex < NUM_SMALL_POOLS);
		}

		Alignment = FPlatformMath::Max<uint32>(Alignment, Alloc.OsAllocationGranularity);
		SizeOut = Align(Count, Alignment);
		check(SizeOut >= Count);
		return SizeOut;
	}

	inline uint32 BoundSizeToPoolIndex(SIZE_T Size, const uint8(&MemSizeToPoolIndex)[SIZE_TO_POOL_INDEX_NUM]) const
	{
		const auto Index = ((Size + UE_MBC_MIN_SMALL_POOL_ALIGNMENT - 1) >> UE_MBC_BIN_SIZE_SHIFT);
		checkSlow(Index >= 0 && Index < SIZE_TO_POOL_INDEX_NUM); // and it should be in the table
		const uint32 PoolIndex = uint32(MemSizeToPoolIndex[Index]);
		checkSlow(PoolIndex >= 0 && PoolIndex < NUM_SMALL_POOLS);
		return PoolIndex;
	}

	// force no inline, so it will not bloat fast code path since this is unlikely to happen
	FORCENOINLINE bool PromoteToLargerBin(SIZE_T& Size, uint32& Alignment, const AllocType& Alloc) const
	{
		// try to promote our allocation request to a larger bin with a matching natural alignment
		// if requested alignment is larger than UE_MBC_STANDARD_ALIGNMENT but smaller than UE_MBC_MAX_SMALL_POOL_ALIGNMENT
		// so we don't do a page allocation with a lot of memory waste
		Alignment = FMath::Max<uint32>(Alignment, UE_MBC_STANDARD_ALIGNMENT);
		const SIZE_T AlignedSize = Align(Size, Alignment);
		if (UNLIKELY((AlignedSize <= MAX_SMALL_POOL_SIZE) && (Alignment <= UE_MBC_MAX_SMALL_POOL_ALIGNMENT)))
		{
			uint32 PoolIndex = BoundSizeToPoolIndex(AlignedSize, Alloc.MemSizeToPoolIndex);
			do
			{
				const uint32 BlockSize = Alloc.PoolIndexToBinSize(PoolIndex);
				if (IsAligned(BlockSize, Alignment))
				{
					// we found a matching pool for our alignment and size requirements, so modify the size request to match
					Size = SIZE_T(BlockSize);
					Alignment = UE_MBC_STANDARD_ALIGNMENT;
					return true;
				}

				PoolIndex++;
			} while (PoolIndex < NUM_SMALL_POOLS);
		}

		return false;
	}

	bool GetAllocationSizeExternal(void* Ptr, SIZE_T& SizeOut)
	{
		if (((AllocType*)this)->GetSmallAllocationSize(Ptr, SizeOut))
		{
			return true;
		}
		if (!Ptr)
		{
			return false;
		}

		typename AllocType::FPoolInfo* Pool;
		{
			//NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned_GetAllocationSizeExternal);
			UE::TUniqueLock Lock(ExternalAllocMutex);
			Pool = Internal::FindPoolInfo(*(AllocType*)this, Ptr);
		}

		if (!Pool)
		{
			UnrecognizedPointerFatalError(Ptr);
		}
		const SIZE_T PoolOsBytes = Pool->GetOsAllocatedBytes();
		const SIZE_T PoolOSRequestedBytes = Pool->GetOSRequestedBytes();
		checkf(PoolOSRequestedBytes <= PoolOsBytes, TEXT("FMallocBinned::GetAllocationSizeExternal %zu %zu"), PoolOSRequestedBytes, PoolOsBytes);
		SizeOut = PoolOsBytes;
		return true;
	}

#if UE_MBC_ALLOCATOR_STATS
	int64 GetTotalAllocatedSmallPoolMemory() const
	{
		//NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned_GetTotalAllocatedSmallPoolMemory);
		int64 FreeBlockAllocatedMemory = 0;
		{
			UE::TUniqueLock Lock(GetFreeBlockListsRegistrationMutex());
			for (const FPerThreadFreeBlockLists* FreeBlockLists : GetRegisteredFreeBlockLists())
			{
				FreeBlockAllocatedMemory += FreeBlockLists->AllocatedMemory;
			}
			FreeBlockAllocatedMemory += ConsolidatedMemory.load(std::memory_order_relaxed);
		}

		return AllocatedSmallPoolMemory.load(std::memory_order_relaxed) + FreeBlockAllocatedMemory;
	}
#endif

	void AllocateHashBuckets()
	{
		const uint64 MaxHashBuckets = PtrToPoolMapping.GetMaxHashBuckets();
		const uint64 HashAllocSize = Align(MaxHashBuckets * sizeof(FPoolHashBucket), OsAllocationGranularity);
		HashBuckets = (FPoolHashBucket*)AllocType::AllocateMetaDataMemory(HashAllocSize);
		UE_MBC_UPDATE_STATS(HashMemory += HashAllocSize);
		verify(HashBuckets);

		DefaultConstructItems<FPoolHashBucket>(HashBuckets, MaxHashBuckets);
	}

private:
	FPoolHashBucket* HashBuckets = nullptr;				// Hash buckets for external allocations, reserved in constructor based on the platform constants like page size and virtual address high\low hints
	FPoolHashBucket* HashBucketFreeList = nullptr;		// Hash buckets for allocations that were allocated outside of the platform constants virtual address high\low hints

	static void RegisterThreadFreeBlockLists(FPerThreadFreeBlockLists* FreeBlockLists)
	{
		//NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned_RegisterThreadFreeBlockLists);
		UE::TUniqueLock Lock(GetFreeBlockListsRegistrationMutex());
		GetRegisteredFreeBlockLists().Add(FreeBlockLists);
	}

	static void UnregisterThreadFreeBlockLists(FPerThreadFreeBlockLists* FreeBlockLists)
	{
		//NOALLOC_SCOPE_CYCLE_COUNTER(STAT_FMallocBinned_UnregisterThreadFreeBlockLists);
		UE::TUniqueLock Lock(GetFreeBlockListsRegistrationMutex());
		GetRegisteredFreeBlockLists().Remove(FreeBlockLists);
		UE_MBC_UPDATE_STATS(ConsolidatedMemory.fetch_add(FreeBlockLists->AllocatedMemory, std::memory_order_relaxed));
	}
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#	include "HAL/CriticalSection.h"
#endif

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#	include "ProfilingDebugging/CsvProfiler.h"
#endif
