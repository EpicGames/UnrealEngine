// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Async/TaskGraphInterfaces.h"
#include "HAL/MallocBinnedCommon.h"
#include "Misc/App.h"
#include "Stats/Stats.h"
#include "FramePro/FrameProProfiler.h"
#include "ProfilingDebugging/CsvProfiler.h"

CSV_DECLARE_CATEGORY_EXTERN(MallocBinned);

#if FRAMEPRO_ENABLED
//	Pushes a profiler scope if it's safe to do so without any new allocations.
	class FNoAllocScopeCycleCounter
	{
	public:
		inline FNoAllocScopeCycleCounter(const ANSICHAR* InStatString)
		{
			if (FFrameProProfiler::IsThreadContextReady() && GCycleStatsShouldEmitNamedEvents)
			{
				StatString = InStatString;
				FFrameProProfiler::PushEvent(StatString);
			}
		}

		inline ~FNoAllocScopeCycleCounter()
		{
			if (StatString)
			{
				FFrameProProfiler::PopEvent(StatString);
			}
		}
	private:
		const ANSICHAR* StatString = nullptr;
	};

#	define NOALLOC_SCOPE_CYCLE_COUNTER(Stat) FNoAllocScopeCycleCounter NoAllocCycleCounter_##Stat(#Stat)
#else
#	define NOALLOC_SCOPE_CYCLE_COUNTER(Stat)
#endif	// ~FRAMEPRO_ENABLED

namespace MallocBinnedPrivate
{
	template<int NumSmallPools>
	struct TGlobalRecycler
	{
		bool PushBundle(uint32 InPoolIndex, FMallocBinnedCommonBase::FBundleNode* InBundle)
		{
			const uint32 NumCachedBundles = FMath::Min<uint32>(GMallocBinnedMaxBundlesBeforeRecycle, UE_DEFAULT_GMallocBinnedMaxBundlesBeforeRecycle);
			for (uint32 Slot = 0; Slot < NumCachedBundles; Slot++)
			{
				if (!Bundles[InPoolIndex].FreeBundles[Slot])
				{
					if (!FPlatformAtomics::InterlockedCompareExchangePointer((void**)&Bundles[InPoolIndex].FreeBundles[Slot], InBundle, nullptr))
					{
						return true;
					}
				}
			}
			return false;
		}

		FMallocBinnedCommonBase::FBundleNode* PopBundle(uint32 InPoolIndex)
		{
			const uint32 NumCachedBundles = FMath::Min<uint32>(GMallocBinnedMaxBundlesBeforeRecycle, UE_DEFAULT_GMallocBinnedMaxBundlesBeforeRecycle);
			for (uint32 Slot = 0; Slot < NumCachedBundles; Slot++)
			{
				FMallocBinnedCommonBase::FBundleNode* Result = Bundles[InPoolIndex].FreeBundles[Slot];
				if (Result)
				{
					if (FPlatformAtomics::InterlockedCompareExchangePointer((void**)&Bundles[InPoolIndex].FreeBundles[Slot], nullptr, Result) == Result)
					{
						return Result;
					}
				}
			}
			return nullptr;
		}

	private:
		struct FBundlePointer
		{
			alignas(PLATFORM_CACHE_LINE_SIZE) FMallocBinnedCommonBase::FBundleNode* FreeBundles[UE_DEFAULT_GMallocBinnedMaxBundlesBeforeRecycle];

			FBundlePointer()
			{
				DefaultConstructItems<FMallocBinnedCommonBase::FBundleNode*>(FreeBundles, UE_DEFAULT_GMallocBinnedMaxBundlesBeforeRecycle);
			}
		};
		static_assert(sizeof(FBundlePointer) == PLATFORM_CACHE_LINE_SIZE, "FBundlePointer should be the same size as a cache line");
		alignas(PLATFORM_CACHE_LINE_SIZE) FBundlePointer Bundles[NumSmallPools];
	};
}


class FMallocBinnedCommonUtils
{
public:
	template <class AllocType>
	static void TrimThreadFreeBlockLists(AllocType& Allocator, typename AllocType::FPerThreadFreeBlockLists* FreeBlockLists)
	{
		if (FreeBlockLists)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FMallocBinnedCommonUtils::TrimThreadFreeBlockLists);

			for (int32 PoolIndex = 0; PoolIndex != AllocType::NUM_SMALL_POOLS; ++PoolIndex)
			{
				typename AllocType::FBundleNode* Bundles = FreeBlockLists->PopBundles(PoolIndex);
				if (Bundles)
				{
					Allocator.FreeBundles(Bundles, PoolIndex);
				}
			}
		}
	}

	template <class AllocType>
	static void FlushCurrentThreadCache(AllocType& Allocator, bool bNewEpochOnly = false)
	{
		if (typename AllocType::FPerThreadFreeBlockLists* Lists = AllocType::FPerThreadFreeBlockLists::Get())
		{
			if (Lists->UpdateEpoch(Allocator.MemoryTrimEpoch.load(std::memory_order_relaxed)) || !bNewEpochOnly)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FMallocBinnedCommonUtils::FlushCurrentThreadCache);
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FMallocBinnedCommonUtils_FlushCurrentThreadCache);

				const double StartTimeInner = FPlatformTime::Seconds();
				TrimThreadFreeBlockLists(Allocator, Lists);
				const double WaitForTrimTime = FPlatformTime::Seconds() - StartTimeInner;

				if (WaitForTrimTime > GMallocBinnedFlushThreadCacheMaxWaitTime)
				{
					UE_LOGF(LogMemory, Warning, "FMalloc%ls took %6.2fms to wait for mutex AND trim.", Allocator.GetDescriptiveName(), WaitForTrimTime * 1000.0f);
				}
			}
		}
	}

	template <class AllocType>
	static void Trim(AllocType& Allocator)
	{
		// Update the trim epoch so that threads cleanup their thread-local memory when going to sleep.
		Allocator.MemoryTrimEpoch.fetch_add(1, std::memory_order_relaxed);

		QUICK_SCOPE_CYCLE_COUNTER(STAT_FMallocBinnedCommonUtils_Trim);

		// Process thread-local memory caches from as many threads as possible without waking them up.
		// Skip on desktop as we may have too many threads and this could cause some hitches.
		if (!PLATFORM_DESKTOP && GMallocBinnedFlushRegisteredThreadCachesOnOneThread != 0)
		{
			UE::TUniqueLock FreeBlockLock(AllocType::GetFreeBlockListsRegistrationMutex());
			for (typename AllocType::FPerThreadFreeBlockLists* BlockList : AllocType::GetRegisteredFreeBlockLists())
			{
				// If we're unable to lock, it's because the thread is currently active so it
				// will do the flush itself when going back to sleep because we incremented the Epoch.
				if (BlockList->TryLock())
				{
					// Only trim if the epoch has been updated, otherwise the thread already
					// did the trimming when it went to sleep.
					if (BlockList->UpdateEpoch(Allocator.MemoryTrimEpoch.load(std::memory_order_relaxed)))
					{
						TrimThreadFreeBlockLists(Allocator, BlockList);
					}
					BlockList->Unlock();
				}
			}
		}

		TFunction<void(ENamedThreads::Type CurrentThread)> Broadcast =
			[&Allocator](ENamedThreads::Type MyThread)
			{
				// We might already have updated the Epoch so we can skip doing anything costly (i.e. Mutex) in that case.
				const bool bNewEpochOnly = true;
				FlushCurrentThreadCache(Allocator, bNewEpochOnly);
			};

		// Skip task threads on desktop platforms as it is too slow and they don't have much memory
		if (PLATFORM_DESKTOP)
		{
			FTaskGraphInterface::BroadcastSlow_OnlyUseForSpecialPurposes(false, false, Broadcast);
		}
		else
		{
			FTaskGraphInterface::BroadcastSlow_OnlyUseForSpecialPurposes(FPlatformProcess::SupportsMultithreading() && FApp::ShouldUseThreadingForPerformance(), false, Broadcast);
		}
	}

	template <class AllocType>
	static void UpdateStatsCommon(const AllocType& Alloc)
	{
#if UE_MBC_ALLOCATOR_STATS && CSV_PROFILER_STATS 
		if (!GMallocBinnedEnableCSVStats && !FCsvProfiler::Get()->IsCategoryEnabled(CSV_CATEGORY_INDEX(MallocBinned)))
		{
			return;
		}

		constexpr int32 NumSmallPools = AllocType::NUM_SMALL_POOLS;

		FCsvProfiler::Get()->EnableCategoryByIndex(CSV_CATEGORY_INDEX(MallocBinned), true);

		static bool bFirstTime = true;
		static FName FragmentationsStats[NumSmallPools];
		static FName WasteStats[NumSmallPools];
		static FName TotalMemStats[NumSmallPools];

		if (bFirstTime)
		{
			for (int32 i = 0; i < NumSmallPools; i++)
			{
				const int BinSize = Alloc.PoolIndexToBinSize(i);

				TCHAR Name[64];
				FCString::Sprintf(Name, TEXT("FragmentationBin%d"), BinSize);
				FragmentationsStats[i] = FName(Name);

				FCString::Sprintf(Name, TEXT("WasteBin%d"), BinSize);
				WasteStats[i] = FName(Name);

				FCString::Sprintf(Name, TEXT("TotalMemBin%d"), BinSize);
				TotalMemStats[i] = FName(Name);
			}

			bFirstTime = false;
		}

		for (int32 i = 0; i < NumSmallPools; i++)
		{
			const float Fragmentation = 1.0f - (float)Alloc.SmallPoolTables[i].TotalUsedBins / (float)Alloc.SmallPoolTables[i].TotalAllocatedBins;
			FCsvProfiler::RecordCustomStat(FragmentationsStats[i], CSV_CATEGORY_INDEX(MallocBinned), int(Fragmentation * 100.0f), ECsvCustomStatOp::Set);

			const float TotalMem = (float)Alloc.SmallPoolTables[i].TotalAllocatedMem / 1024.0f / 1024.0f;
			FCsvProfiler::RecordCustomStat(TotalMemStats[i], CSV_CATEGORY_INDEX(MallocBinned), TotalMem, ECsvCustomStatOp::Set);

			FCsvProfiler::RecordCustomStat(WasteStats[i], CSV_CATEGORY_INDEX(MallocBinned), TotalMem * Fragmentation, ECsvCustomStatOp::Set);
		}

		CSV_CUSTOM_STAT(MallocBinned, RequestedSmallPoolMemoryMB, (float)Alloc.GetTotalAllocatedSmallPoolMemory() / (1024.0f * 1024.0f), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(MallocBinned, TotalAllocatedSmallPoolMemoryMB, (float)Alloc.AllocatedOSSmallPoolMemory.load(std::memory_order_relaxed) / (1024.0f * 1024.0f), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(MallocBinned, RequestedLargeAllocsMemoryMB, (float)Alloc.AllocatedLargePoolMemory.load(std::memory_order_relaxed) / (1024.0f * 1024.0f), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(MallocBinned, TotalAllocatedLargeAllocsMemoryMB, (float)Alloc.AllocatedLargePoolMemoryWAlignment.load(std::memory_order_relaxed) / (1024.0f * 1024.0f), ECsvCustomStatOp::Set);
#endif
	}
};
