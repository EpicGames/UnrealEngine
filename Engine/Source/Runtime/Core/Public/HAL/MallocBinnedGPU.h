// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PlatformMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/Atomic.h"
#include "Templates/MemoryOps.h"

#if PLATFORM_HAS_FPlatformVirtualMemoryBlock
#include "Async/UniqueLock.h"
#include "HAL/Allocators/CachedOSPageAllocator.h"
#include "HAL/Allocators/PooledVirtualMemoryAllocator.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/MallocBinnedCommon.h"
#include "HAL/MemoryBase.h"
#include "HAL/PlatformMath.h"
#include "HAL/PlatformMutex.h"
#include "HAL/PlatformTLS.h"
#include "HAL/UnrealMemory.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeLock.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/Function.h"


#define BINNEDGPU_MAX_GMallocBinnedGPUMaxBundlesBeforeRecycle (8)

#define COLLECT_BINNEDGPU_STATS (!UE_BUILD_SHIPPING)

#if COLLECT_BINNEDGPU_STATS
	#define MBG_STAT(x) x
#else
	#define MBG_STAT(x)
#endif

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

struct FArenaParams
{
	// these are parameters you set
	uint64 AddressLimit = 1024 * 1024 * 1024; // this controls the size of the root hash table
	uint32 BasePageSize = 4096; // this is used to make sensible calls to malloc and figures into the standard pool sizes if bUseStandardSmallPoolSizes is true
	uint32 AllocationGranularity = 4096; // this is the granularity of the commit and decommit calls used on the VM slabs
	uint32 MaxSizePerBundle = 8192;
	uint32 MaxStandardPoolSize = 128 * 1024; // these are added to the standard pool sizes, mainly to use the TLS caches, they are typically one block per slab
	uint16 MaxBlocksPerBundle = 64;
	uint8 MaxMemoryPerBlockSizeShift = 29;
	uint8 EmptyCacheAllocExtra = 32;
	uint8 MaxGlobalBundles = 32;
	uint8 MinimumAlignmentShift = 4;
	uint8 PoolCount;
	bool bUseSeparateVMPerPool = false;
	bool bPerThreadCaches = true;
	bool bUseStandardSmallPoolSizes = true;
	bool bAttemptToAlignSmallBocks = true;
	TArray<uint32> AdditionalBlockSizes;

	// This lambdas is similar to the platform virtual memory HAL and by default just call that. 
	TFunction<FPlatformMemory::FPlatformVirtualMemoryBlock(SIZE_T)> ReserveVM;

	// These allow you to override the large block allocator. The value add here is that MBA tracks the metadata for you and call tell the difference between a large block pointer and a small block pointer.
	// By defaults these just use the platform VM interface to allocate some committed memory
	TFunction<void* (SIZE_T, SIZE_T, SIZE_T&, uint32&)> LargeBlockAlloc;
	TFunction<void(void*, uint32)> LargeBlockFree;

	// these are parameters are derived from other parameters
	uint64 MaxMemoryPerBlockSize;
	uint32 MaxPoolSize;
	uint32 MinimumAlignment;
	uint32 MaximumAlignmentForSmallBlock;
};

class FMallocBinnedGPU final : public FMalloc
{
	struct FGlobalRecycler;
	struct FPoolInfoLarge;
	struct FPoolInfoSmall;
	struct FPoolTable;
	struct PoolHashBucket;
	struct Private;


	struct FGPUMemoryBlockProxy
	{
		uint8 MemoryModifiedByCPU[32 - sizeof(void*)]; // might be modified for free list links, etc
		void *GPUMemory;  // pointer to the actual GPU memory, which we cannot modify with the CPU

		FGPUMemoryBlockProxy(void *InGPUMemory)
			: GPUMemory(InGPUMemory)
		{
			check(GPUMemory);
		}
	};

	struct FFreeBlock
	{
		enum
		{
			CANARY_VALUE = 0xc3
		};

		inline FFreeBlock(uint32 InPageSize, uint32 InBlockSize, uint32 InPoolIndex, uint8 MinimumAlignmentShift)
			: BlockSizeShifted(InBlockSize >> MinimumAlignmentShift)
			, PoolIndex(InPoolIndex)
			, Canary(CANARY_VALUE)
			, NextFreeBlock(nullptr)
		{
			check(InPoolIndex < MAX_uint8 && (InBlockSize >> MinimumAlignmentShift) <= MAX_uint16);
			NumFreeBlocks = InPageSize / InBlockSize;
		}

		UE_FORCEINLINE_HINT uint32 GetNumFreeRegularBlocks() const
		{
			return NumFreeBlocks;
		}
		UE_FORCEINLINE_HINT bool IsCanaryOk() const
		{
			return Canary == FFreeBlock::CANARY_VALUE;
		}

		inline void CanaryTest() const
		{
			if (!IsCanaryOk())
			{
				CanaryFail();
			}
		}
		void CanaryFail() const;

		inline void* AllocateRegularBlock(uint8 MinimumAlignmentShift)
		{
			--NumFreeBlocks;
			return (uint8*)(((FGPUMemoryBlockProxy*)this)->GPUMemory) + NumFreeBlocks * (uint32(BlockSizeShifted) << MinimumAlignmentShift);
		}

		uint16 BlockSizeShifted;		// Size of the blocks that this list points to >> ArenaParams.MinimumAlignmentShift
		uint8 PoolIndex;				// Index of this pool
		uint8 Canary;					// Constant value of 0xe3
		uint32 NumFreeBlocks;          // Number of consecutive free blocks here, at least 1.
		FFreeBlock* NextFreeBlock;     // Next free block or nullptr
	};

	struct FPoolTable
	{
		uint32 BlockSize;
		uint16 BlocksPerBlockOfBlocks;
		uint8 PagesPlatformForBlockOfBlocks;

		FBitTree BlockOfBlockAllocationBits; // one bits in here mean the virtual memory is committed
		FBitTree BlockOfBlockIsExhausted;    // one bit in here means the pool is completely full

		uint32 NumEverUsedBlockOfBlocks;
		FPoolInfoSmall** PoolInfos;

		uint64 UnusedAreaOffsetLow;
	};

	struct FPtrToPoolMapping
	{
		FPtrToPoolMapping()
			: PtrToPoolPageBitShift(0)
			, HashKeyShift(0)
			, PoolMask(0)
			, MaxHashBuckets(0)
		{
		}
		explicit FPtrToPoolMapping(uint32 InPageSize, uint64 InNumPoolsPerPage, uint64 AddressLimit)
		{
			Init(InPageSize, InNumPoolsPerPage, AddressLimit);
		}

		void Init(uint32 InPageSize, uint64 InNumPoolsPerPage, uint64 AddressLimit)
		{
			uint64 PoolPageToPoolBitShift = FPlatformMath::CeilLogTwo(InNumPoolsPerPage);

			PtrToPoolPageBitShift = FPlatformMath::CeilLogTwo(InPageSize);
			HashKeyShift = PtrToPoolPageBitShift + PoolPageToPoolBitShift;
			PoolMask = (1ull << PoolPageToPoolBitShift) - 1;
			MaxHashBuckets = AddressLimit >> HashKeyShift;
		}

		inline void GetHashBucketAndPoolIndices(const void* InPtr, uint32& OutBucketIndex, UPTRINT& OutBucketCollision, uint32& OutPoolIndex) const
		{
			OutBucketCollision = (UPTRINT)InPtr >> HashKeyShift;
			OutBucketIndex = uint32(OutBucketCollision & (MaxHashBuckets - 1));
			OutPoolIndex = ((UPTRINT)InPtr >> PtrToPoolPageBitShift) & PoolMask;
		}

		UE_FORCEINLINE_HINT uint64 GetMaxHashBuckets() const
		{
			return MaxHashBuckets;
		}

	private:
		/** Shift to apply to a pointer to get the reference from the indirect tables */
		uint64 PtrToPoolPageBitShift;
		/** Shift required to get required hash table key. */
		uint64 HashKeyShift;
		/** Used to mask off the bits that have been used to lookup the indirect table */
		uint64 PoolMask;
		// PageSize dependent constants
		uint64 MaxHashBuckets;
	};

	struct FBundleNode
	{
		FBundleNode* NextNodeInCurrentBundle;
		union
		{
			FBundleNode* NextBundle;
			int32 Count;
		};
	};

	struct FBundle
	{
		UE_FORCEINLINE_HINT FBundle()
		{
			Reset();
		}

		inline void Reset()
		{
			Head = nullptr;
			Count = 0;
		}

		inline void PushHead(FBundleNode* Node)
		{
			Node->NextNodeInCurrentBundle = Head;
			Node->NextBundle = nullptr;
			Head = Node;
			Count++;
		}

		inline FBundleNode* PopHead()
		{
			FBundleNode* Result = Head;

			Count--;
			Head = Head->NextNodeInCurrentBundle;
			return Result;
		}

		FBundleNode* Head;
		uint32       Count;
	};

	struct FFreeBlockList
	{
		// return true if we actually pushed it
		inline bool PushToFront(FMallocBinnedGPU& Allocator, void* InPtr, uint32 InPoolIndex, uint32 InBlockSize, const FArenaParams& LocalArenaParams)
		{
			check(InPtr);

			if ((PartialBundle.Count >= (uint32)LocalArenaParams.MaxBlocksPerBundle) | (PartialBundle.Count * InBlockSize >= (uint32)LocalArenaParams.MaxSizePerBundle))
			{
				if (FullBundle.Head)
				{
					return false;
				}
				FullBundle = PartialBundle;
				PartialBundle.Reset();
			}
			PartialBundle.PushHead((FBundleNode*)new FGPUMemoryBlockProxy(InPtr));
			MBG_STAT(Allocator.GPUProxyMemory += sizeof(FGPUMemoryBlockProxy);)
			return true;
		}
		UE_FORCEINLINE_HINT bool CanPushToFront(uint32 InPoolIndex, uint32 InBlockSize, const FArenaParams& LocalArenaParams)
		{
			return !((!!FullBundle.Head) & ((PartialBundle.Count >= (uint32)LocalArenaParams.MaxBlocksPerBundle) | (PartialBundle.Count * InBlockSize >= (uint32)LocalArenaParams.MaxSizePerBundle)));
		}
		inline void* PopFromFront(FMallocBinnedGPU& Allocator, uint32 InPoolIndex)
		{
			if ((!PartialBundle.Head) & (!!FullBundle.Head))
			{
				PartialBundle = FullBundle;
				FullBundle.Reset();
			}
			void *Result = nullptr;
			if (PartialBundle.Head)
			{
				FGPUMemoryBlockProxy* Proxy = (FGPUMemoryBlockProxy*)PartialBundle.PopHead();
				Result = Proxy->GPUMemory;
				check(Result);
				delete Proxy;
				MBG_STAT(Allocator.GPUProxyMemory -= sizeof(FGPUMemoryBlockProxy);)
			}
			return Result;
		}

		// tries to recycle the full bundle, if that fails, it is returned for freeing
		FBundleNode* RecyleFull(FArenaParams& LocalArenaParams, FGlobalRecycler& GGlobalRecycler, uint32 InPoolIndex);
		bool ObtainPartial(FArenaParams& LocalArenaParams, FGlobalRecycler& GGlobalRecycler, uint32 InPoolIndex);
		FBundleNode* PopBundles(uint32 InPoolIndex);
	private:
		FBundle PartialBundle;
		FBundle FullBundle;
	};

	struct FPerThreadFreeBlockLists
	{
		UE_FORCEINLINE_HINT static FPerThreadFreeBlockLists* Get(uint32 BinnedGPUTlsSlot)
		{
			return FPlatformTLS::IsValidTlsSlot(BinnedGPUTlsSlot) ? (FPerThreadFreeBlockLists*)FPlatformTLS::GetTlsValue(BinnedGPUTlsSlot) : nullptr;
		}
		static void SetTLS(FMallocBinnedGPU& Allocator);
		static int64 ClearTLS(FMallocBinnedGPU& Allocator);

		FPerThreadFreeBlockLists(uint32 PoolCount)
			: AllocatedMemory(0)
		{ 
			FreeLists.AddDefaulted(PoolCount);
		}

		UE_FORCEINLINE_HINT void* Malloc(FMallocBinnedGPU& Allocator, uint32 InPoolIndex)
		{
			return FreeLists[InPoolIndex].PopFromFront(Allocator, InPoolIndex);
		}
		// return true if the pointer was pushed
		UE_FORCEINLINE_HINT bool Free(FMallocBinnedGPU& Allocator, void* InPtr, uint32 InPoolIndex, uint32 InBlockSize, const FArenaParams& LocalArenaParams)
		{
			return FreeLists[InPoolIndex].PushToFront(Allocator, InPtr, InPoolIndex, InBlockSize, LocalArenaParams);
		}
		// return true if a pointer can be pushed
		UE_FORCEINLINE_HINT bool CanFree(uint32 InPoolIndex, uint32 InBlockSize, const FArenaParams& LocalArenaParams)
		{
			return FreeLists[InPoolIndex].CanPushToFront(InPoolIndex, InBlockSize, LocalArenaParams);
		}
		// returns a bundle that needs to be freed if it can't be recycled
		FBundleNode* RecycleFullBundle(FArenaParams& LocalArenaParams, FGlobalRecycler& GlobalRecycler, uint32 InPoolIndex)
		{
			return FreeLists[InPoolIndex].RecyleFull(LocalArenaParams, GlobalRecycler, InPoolIndex);
		}
		// returns true if we have anything to pop
		bool ObtainRecycledPartial(FArenaParams& LocalArenaParams, FGlobalRecycler& GlobalRecycler, uint32 InPoolIndex)
		{
			return FreeLists[InPoolIndex].ObtainPartial(LocalArenaParams, GlobalRecycler, InPoolIndex);
		}
		FBundleNode* PopBundles(uint32 InPoolIndex)
		{
			return FreeLists[InPoolIndex].PopBundles(InPoolIndex);
		}
		int64 AllocatedMemory;
		TArray<FFreeBlockList> FreeLists;
	};

	struct FGlobalRecycler
	{
		void Init(uint32 PoolCount)
		{
			Bundles.AddDefaulted(PoolCount);
		}
		bool PushBundle(uint32 NumCachedBundles, uint32 InPoolIndex, FBundleNode* InBundle)
		{
			for (uint32 Slot = 0; Slot < NumCachedBundles && Slot < BINNEDGPU_MAX_GMallocBinnedGPUMaxBundlesBeforeRecycle; Slot++)
			{
				if (!Bundles[InPoolIndex].FreeBundles[Slot])
				{
					if (!FPlatformAtomics::InterlockedCompareExchangePointer((void**)&Bundles[InPoolIndex].FreeBundles[Slot], InBundle, nullptr))
					{
						return true;
					}
				}
			}
			return false;
		}

		FBundleNode* PopBundle(uint32 NumCachedBundles, uint32 InPoolIndex)
		{
			for (uint32 Slot = 0; Slot < NumCachedBundles && Slot < BINNEDGPU_MAX_GMallocBinnedGPUMaxBundlesBeforeRecycle; Slot++)
			{
				FBundleNode* Result = Bundles[InPoolIndex].FreeBundles[Slot];
				if (Result)
				{
					if (FPlatformAtomics::InterlockedCompareExchangePointer((void**)&Bundles[InPoolIndex].FreeBundles[Slot], nullptr, Result) == Result)
					{
						return Result;
					}
				}
			}
			return nullptr;
		}

	private:
		struct FPaddedBundlePointer
		{
			FBundleNode* FreeBundles[BINNEDGPU_MAX_GMallocBinnedGPUMaxBundlesBeforeRecycle];
			FPaddedBundlePointer()
			{
				DefaultConstructItems<FBundleNode*>(FreeBundles, BINNEDGPU_MAX_GMallocBinnedGPUMaxBundlesBeforeRecycle);
			}
		};
		TArray<FPaddedBundlePointer> Bundles;
	};


	inline uint64 PoolIndexFromPtr(const void* Ptr) 
	{
		if (PoolSearchDiv == 0)
		{
			return (UPTRINT(Ptr) - UPTRINT(PoolBaseVMPtr[0])) >> ArenaParams.MaxMemoryPerBlockSizeShift;
		}
		uint64 PoolIndex = ArenaParams.PoolCount;
		if (((uint8*)Ptr >= PoolBaseVMPtr[0]) & ((uint8*)Ptr < HighestPoolBaseVMPtr + ArenaParams.MaxMemoryPerBlockSize))
		{
			PoolIndex = uint64((uint8*)Ptr - PoolBaseVMPtr[0]) / PoolSearchDiv;
			if (PoolIndex >= ArenaParams.PoolCount)
			{
				PoolIndex = ArenaParams.PoolCount - 1;
			}
			if ((uint8*)Ptr < PoolBaseVMPtr[(int32)PoolIndex])
			{
				do
				{
					PoolIndex--;
					check(PoolIndex < ArenaParams.PoolCount);
				} while ((uint8*)Ptr < PoolBaseVMPtr[(int32)PoolIndex]);
				if ((uint8*)Ptr >= PoolBaseVMPtr[(int32)PoolIndex] + ArenaParams.MaxMemoryPerBlockSize)
				{
					PoolIndex = ArenaParams.PoolCount; // was in the gap
				}
			}
			else if ((uint8*)Ptr >= PoolBaseVMPtr[(int32)PoolIndex] + ArenaParams.MaxMemoryPerBlockSize)
			{
				do
				{
					PoolIndex++;
					check(PoolIndex < ArenaParams.PoolCount);
				} while ((uint8*)Ptr >= PoolBaseVMPtr[(int32)PoolIndex] + ArenaParams.MaxMemoryPerBlockSize);
				if ((uint8*)Ptr < PoolBaseVMPtr[(int32)PoolIndex])
				{
					PoolIndex = ArenaParams.PoolCount; // was in the gap
				}
			}
		}
		return PoolIndex;
	}

	UE_FORCEINLINE_HINT uint8* PoolBasePtr(uint32 InPoolIndex)
	{
		return PoolBaseVMPtr[InPoolIndex];
	}
	inline uint64 PoolIndexFromPtrChecked(const void* Ptr)
	{
		uint64 Result = PoolIndexFromPtr(Ptr);
		check(Result < ArenaParams.PoolCount);
		return Result;
	}

	UE_FORCEINLINE_HINT bool IsOSAllocation(const void* Ptr)
	{
		return PoolIndexFromPtr(Ptr) >= ArenaParams.PoolCount;
	}


	inline void* BlockOfBlocksPointerFromContainedPtr(const void* Ptr, uint8 PagesPlatformForBlockOfBlocks, uint32& OutBlockOfBlocksIndex)
	{
		uint32 PoolIndex = PoolIndexFromPtrChecked(Ptr);
		uint8* PoolStart = PoolBasePtr(PoolIndex);
		uint64 BlockOfBlocksIndex = (UPTRINT(Ptr) - UPTRINT(PoolStart)) / (UPTRINT(PagesPlatformForBlockOfBlocks) * UPTRINT(ArenaParams.AllocationGranularity));
		OutBlockOfBlocksIndex = BlockOfBlocksIndex;

		uint8* Result = PoolStart + BlockOfBlocksIndex * UPTRINT(PagesPlatformForBlockOfBlocks) * UPTRINT(ArenaParams.AllocationGranularity);

		check(Result < PoolStart + ArenaParams.MaxMemoryPerBlockSize);
		return Result;
	}
	inline uint8* BlockPointerFromIndecies(uint32 InPoolIndex, uint32 BlockOfBlocksIndex, uint32 BlockOfBlocksSize)
	{
		uint8* PoolStart = PoolBasePtr(InPoolIndex);
		uint8* Ptr = PoolStart + BlockOfBlocksIndex * uint64(BlockOfBlocksSize);
		check(Ptr + BlockOfBlocksSize <= PoolStart + ArenaParams.MaxMemoryPerBlockSize);
		return Ptr;
	}
	CORE_API FPoolInfoSmall* PushNewPoolToFront(FMallocBinnedGPU& Allocator, uint32 InBlockSize, uint32 InPoolIndex, uint32& OutBlockOfBlocksIndex);
	CORE_API FPoolInfoSmall* GetFrontPool(FPoolTable& Table, uint32 InPoolIndex, uint32& OutBlockOfBlocksIndex);

	inline bool AdjustSmallBlockSizeForAlignment(SIZE_T& InOutSize, uint32 Alignment)
	{
		if ((InOutSize <= ArenaParams.MaxPoolSize) & (Alignment <= ArenaParams.MinimumAlignment)) // one branch, not two
		{
			return true;
		}
		SIZE_T AlignedSize = Align(InOutSize, Alignment);
		if (ArenaParams.bAttemptToAlignSmallBocks & (AlignedSize <= ArenaParams.MaxPoolSize) & (Alignment <= ArenaParams.MaximumAlignmentForSmallBlock)) // one branch, not three
		{
			uint32 PoolIndex = BoundSizeToPoolIndex(AlignedSize);
			while (true)
			{
				uint32 BlockSize = PoolIndexToBlockSize(PoolIndex);
				if (IsAligned(BlockSize, Alignment))
				{
					InOutSize = SIZE_T(BlockSize);
					return true;
				}
				PoolIndex++;
				check(PoolIndex < ArenaParams.PoolCount);
			}
		}
		return false;
	}

public:


	CORE_API FMallocBinnedGPU();
	FArenaParams& GetParams()
	{
		return ArenaParams;
	}
	CORE_API void InitMallocBinned();

	CORE_API virtual ~FMallocBinnedGPU();


	// FMalloc interface.
	CORE_API virtual bool IsInternallyThreadSafe() const override;
	inline virtual void* Malloc(SIZE_T Size, uint32 Alignment) override
	{
		Alignment = FMath::Max<uint32>(Alignment, ArenaParams.MinimumAlignment);

		void* Result = nullptr;

		// Only allocate from the small pools if the size is small enough and the alignment isn't crazy large.
		// With large alignments, we'll waste a lot of memory allocating an entire page, but such alignments are highly unlikely in practice.
		if (AdjustSmallBlockSizeForAlignment(Size, Alignment))
		{
			FPerThreadFreeBlockLists* Lists = ArenaParams.bPerThreadCaches ? FPerThreadFreeBlockLists::Get(BinnedGPUTlsSlot) : nullptr;
			if (Lists)
			{
				uint32 PoolIndex = BoundSizeToPoolIndex(Size);
				uint32 BlockSize = PoolIndexToBlockSize(PoolIndex);
				Result = Lists->Malloc(*this, PoolIndex);
				if (Result)
				{
					Lists->AllocatedMemory += BlockSize;
					checkSlow(IsAligned(Result, Alignment));
				}
			}
		}
		if (Result == nullptr)
		{
			Result = MallocExternal(Size, Alignment);
		}

		return Result;
	}
	inline virtual void* Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override
	{
		check(!"MallocBinnedGPU cannot realloc memory because the memory is assumed to not be writable by the CPU");
		return nullptr;
	}

	inline virtual void Free(void* Ptr) override
	{
		uint64 PoolIndex = PoolIndexFromPtr(Ptr);
		if (PoolIndex < ArenaParams.PoolCount)
		{
			FPerThreadFreeBlockLists* Lists = ArenaParams.bPerThreadCaches ? FPerThreadFreeBlockLists::Get(BinnedGPUTlsSlot) : nullptr;
			if (Lists)
			{
				int32 BlockSize = PoolIndexToBlockSize(PoolIndex);
				if (Lists->Free(*this, Ptr, PoolIndex, BlockSize, ArenaParams))
				{
					Lists->AllocatedMemory -= BlockSize;
					return;
				}
			}
		}
		FreeExternal(Ptr);
	}
	inline virtual bool GetAllocationSize(void *Ptr, SIZE_T &SizeOut) override
	{
		uint64 PoolIndex = PoolIndexFromPtr(Ptr);
		if (PoolIndex < ArenaParams.PoolCount)
		{
			SizeOut = PoolIndexToBlockSize(PoolIndex);
			return true;
		}
		return GetAllocationSizeExternal(Ptr, SizeOut);
	}

	inline virtual SIZE_T QuantizeSize(SIZE_T Count, uint32 Alignment) override
	{
		check(DEFAULT_ALIGNMENT <= ArenaParams.MinimumAlignment); // used below
		checkSlow((Alignment & (Alignment - 1)) == 0); // Check the alignment is a power of two
		SIZE_T SizeOut;
		if ((Count <= ArenaParams.MaxPoolSize) & (Alignment <= ArenaParams.MinimumAlignment)) // one branch, not two
		{
			SizeOut = PoolIndexToBlockSize(BoundSizeToPoolIndex(Count));
		}
		else
		{
			Alignment = FPlatformMath::Max<uint32>(Alignment, ArenaParams.AllocationGranularity);
			SizeOut = Align(Count, Alignment);
		}
		check(SizeOut >= Count);
		return SizeOut;
	}

	CORE_API virtual bool ValidateHeap() override;
	CORE_API virtual void Trim(bool bTrimThreadCaches) override;
	CORE_API virtual void SetupTLSCachesOnCurrentThread() override;
	CORE_API virtual void ClearAndDisableTLSCachesOnCurrentThread() override;
	CORE_API virtual const TCHAR* GetDescriptiveName() override;
	// End FMalloc interface.

	CORE_API void FlushCurrentThreadCache();
	CORE_API void* MallocExternal(SIZE_T Size, uint32 Alignment);
	CORE_API void FreeExternal(void *Ptr);
	CORE_API bool GetAllocationSizeExternal(void* Ptr, SIZE_T& SizeOut);

	MBG_STAT(int64 GetTotalAllocatedSmallPoolMemory();)
	CORE_API virtual void GetAllocatorStats(FGenericMemoryStats& out_Stats) override;
	/** Dumps current allocator stats to the log. */
	CORE_API virtual void DumpAllocatorStats(class FOutputDevice& Ar) override;

	inline uint32 BoundSizeToPoolIndex(SIZE_T Size)
	{
		auto Index = ((Size + ArenaParams.MinimumAlignment - 1) >> ArenaParams.MinimumAlignmentShift);
		checkSlow(Index >= 0 && Index <= (ArenaParams.MaxPoolSize >> ArenaParams.MinimumAlignmentShift)); // and it should be in the table
		uint32 PoolIndex = uint32(MemSizeToIndex[Index]);
		checkSlow(PoolIndex >= 0 && PoolIndex < ArenaParams.PoolCount);
		return PoolIndex;
	}
	UE_FORCEINLINE_HINT uint32 PoolIndexToBlockSize(uint32 PoolIndex)
	{
		return uint32(SmallBlockSizesReversedShifted[ArenaParams.PoolCount - PoolIndex - 1]) << ArenaParams.MinimumAlignmentShift;
	}

	CORE_API void Commit(uint32 InPoolIndex, void *Ptr, SIZE_T Size);
	CORE_API void Decommit(uint32 InPoolIndex, void *Ptr, SIZE_T Size);


	// Pool tables for different pool sizes
	TArray<FPoolTable> SmallPoolTables;

	uint32 SmallPoolInfosPerPlatformPage;

	PoolHashBucket* HashBuckets;
	PoolHashBucket* HashBucketFreeList;
	uint64 NumLargePoolsPerPage;

	UE::FPlatformRecursiveMutex Mutex;
	FGlobalRecycler GGlobalRecycler;
	FPtrToPoolMapping PtrToPoolMapping;

	FArenaParams ArenaParams;

	TArray<uint16> SmallBlockSizesReversedShifted; // this is reversed to get the smallest elements on our main cache line
	uint32 BinnedGPUTlsSlot = FPlatformTLS::InvalidTlsSlot;
	uint64 PoolSearchDiv; // if this is zero, the VM turned out to be contiguous anyway so we use a simple subtract and shift
	uint8* HighestPoolBaseVMPtr; // this is a duplicate of PoolBaseVMPtr[ArenaParams.PoolCount - 1]
	FPlatformMemory::FPlatformVirtualMemoryBlock PoolBaseVMBlock;
	TArray<uint8*> PoolBaseVMPtr;
	TArray<FPlatformMemory::FPlatformVirtualMemoryBlock> PoolBaseVMBlocks;
	// Mapping of sizes to small table indices
	TArray<uint8> MemSizeToIndex;

	MBG_STAT(
		int64 BinnedGPUAllocatedSmallPoolMemory = 0; // memory that's requested to be allocated by the game
		int64 BinnedGPUAllocatedOSSmallPoolMemory = 0;

		int64 BinnedGPUAllocatedLargePoolMemory = 0; // memory requests to the OS which don't fit in the small pool
		int64 BinnedGPUAllocatedLargePoolMemoryWAlignment = 0; // when we allocate at OS level we need to align to a size

		int64 BinnedGPUPoolInfoMemory = 0;
		int64 BinnedGPUHashMemory = 0;
		int64 BinnedGPUFreeBitsMemory = 0;
		int64 BinnedGPUTLSMemory = 0;
		TAtomic<int64> ConsolidatedMemory;
		TAtomic<int64> GPUProxyMemory;
	)

	UE::FPlatformRecursiveMutex FreeBlockListsRegistrationMutex;
	UE::FPlatformRecursiveMutex& GetFreeBlockListsRegistrationMutex()
	{
		return FreeBlockListsRegistrationMutex;
	}
	TArray<FPerThreadFreeBlockLists*> RegisteredFreeBlockLists;
	TArray<FPerThreadFreeBlockLists*>& GetRegisteredFreeBlockLists()
	{
		return RegisteredFreeBlockLists;
	}
	void RegisterThreadFreeBlockLists(FPerThreadFreeBlockLists* FreeBlockLists)
	{
		UE::TUniqueLock Lock(GetFreeBlockListsRegistrationMutex());
		GetRegisteredFreeBlockLists().Add(FreeBlockLists);
	}
	int64 UnregisterThreadFreeBlockLists(FPerThreadFreeBlockLists* FreeBlockLists)
	{
		UE::TUniqueLock Lock(GetFreeBlockListsRegistrationMutex());
		GetRegisteredFreeBlockLists().Remove(FreeBlockLists);
		return FreeBlockLists->AllocatedMemory;
	}

	TArray<void*> MallocedPointers;
};

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#	include "HAL/CriticalSection.h"
#endif

#endif
