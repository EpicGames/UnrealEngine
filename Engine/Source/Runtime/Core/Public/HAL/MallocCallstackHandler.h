// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/MemoryBase.h"
#include "HAL/PlatformTLS.h"
#include "Misc/Crc.h"

struct FGenericMemoryStats;

class FMallocCallstackHandler : public FMalloc
{
public:
	CORE_API FMallocCallstackHandler(FMalloc* InMalloc);

	/**
	 * Malloc
	 */
	CORE_API virtual void* Malloc(SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT) override;

	/**
	 * Realloc
	 */
	CORE_API virtual void* Realloc(void* Original, SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT)  override;

	/**
	 * Free
	 */
	CORE_API virtual void Free(void* Original) override;

	/**
	* For some allocators this will return the actual size that should be requested to eliminate
	* internal fragmentation. The return value will always be >= Count. This can be used to grow
	* and shrink containers to optimal sizes.
	* This call is always fast and thread safe with no locking.
	*/
	virtual SIZE_T QuantizeSize(SIZE_T Count, uint32 Alignment)  override
	{
		return UsedMalloc->QuantizeSize(Count, Alignment);
	}

	/**
	* If possible determine the size of the memory allocated at the given address
	*
	* @param Original - Pointer to memory we are checking the size of
	* @param SizeOut - If possible, this value is set to the size of the passed in pointer
	* @return true if succeeded
	*/
	virtual bool GetAllocationSize(void *Original, SIZE_T &SizeOut)  override
	{
		return UsedMalloc->GetAllocationSize(Original, SizeOut);
	}

	/**
	* Releases as much memory as possible. Must be called from the main thread.
	*/
	virtual void Trim(bool bTrimThreadCaches) override
	{
		UsedMalloc->Trim(bTrimThreadCaches);
	}

	/**
	* Set up TLS caches on the current thread. These are the threads that we can trim.
	*/
	virtual void SetupTLSCachesOnCurrentThread() override
	{
		UsedMalloc->SetupTLSCachesOnCurrentThread();
	}

	/**
	* Mark TLS caches for the current thread as used. Thread has woken up to do some processing and needs its TLS caches back.
	*/
	virtual void MarkTLSCachesAsUsedOnCurrentThread() override
	{
		UsedMalloc->MarkTLSCachesAsUsedOnCurrentThread();
	}

	/**
	* Mark TLS caches for current thread as unused. Typically before going to sleep. These are the threads that we can trim without waking them up.
	*/
	virtual void MarkTLSCachesAsUnusedOnCurrentThread()
	{
		UsedMalloc->MarkTLSCachesAsUnusedOnCurrentThread();
	}

	/**
	* Clears the TLS caches on the current thread and disables any future caching.
	*/
	virtual void ClearAndDisableTLSCachesOnCurrentThread() override
	{
		UsedMalloc->ClearAndDisableTLSCachesOnCurrentThread();
	}

	/**
	*	Initializes stats metadata. We need to do this as soon as possible, but cannot be done in the constructor
	*	due to the FName::StaticInit
	*/
	virtual void InitializeStatsMetadata() override
	{
		UsedMalloc->InitializeStatsMetadata();
	}

	/** Called once per frame, gathers and sets all memory allocator statistics into the corresponding stats. MUST BE THREAD SAFE. */
	virtual void UpdateStats() override
	{
		UsedMalloc->UpdateStats();
	}

	/** Writes allocator stats from the last update into the specified destination. */
	virtual void GetAllocatorStats(FGenericMemoryStats& out_Stats) override
	{
		UsedMalloc->GetAllocatorStats(out_Stats);
	}

	/** Dumps current allocator stats to the log. */
	virtual void DumpAllocatorStats(class FOutputDevice& Ar) override
	{
		UsedMalloc->DumpAllocatorStats(Ar);
	}

	/**
	 * Returns if the allocator is guaranteed to be thread-safe and therefore
	 * doesn't need a unnecessary thread-safety wrapper around it.
	 */
	virtual bool IsInternallyThreadSafe() const override
	{
		return true;
	}

	/**
	 * Validates the allocator's heap
	 */
	virtual bool ValidateHeap() override
	{
		return UsedMalloc->ValidateHeap();
	}

	/**
	 * Gets descriptive name for logging purposes.
	 *
	 * @return pointer to human-readable malloc name
	 */
	virtual const TCHAR* GetDescriptiveName() override
	{
		return UsedMalloc->GetDescriptiveName();
	}
		
	virtual void OnMallocInitialized() override
	{
		UsedMalloc->OnMallocInitialized();
	}

	virtual void OnPreFork() override
	{
		UsedMalloc->OnPreFork();
	}

	virtual void OnPostFork() override
	{
		UsedMalloc->OnPostFork();
	}

	static constexpr inline SIZE_T MaxCallStackDepth = 64;
	static constexpr inline SIZE_T CallStackEntriesToSkipCount = 2;

	struct FCallStackInfo
	{
		uint32 Count;
		uint64 FramePointers[MaxCallStackDepth];
	};

	/** Used as a key in our current allocations/freed allocations maps*/
	struct FCallStackMapKey
	{
		uint32 CRC;
		uint64* CallStack;

		FCallStackMapKey(uint64* InCallStack)
			: CallStack(InCallStack)
		{
			CRC = FCrc::MemCrc32(InCallStack, MaxCallStackDepth * sizeof(uint64));
		}

		friend bool operator==(const FCallStackMapKey& A, const FCallStackMapKey& B)
		{
			if (A.CRC != B.CRC)
			{
				return false;
			}
			for (int i = 0; i < MaxCallStackDepth; ++i)
			{
				uint64 APtr = A.CallStack[i];
				uint64 BPtr = B.CallStack[i];
				if (APtr != BPtr)
				{
					return false;
				}
				if (APtr == 0)
					break;
			}
			return true;
		}

		friend inline uint32 GetTypeHash(const FCallStackMapKey& InKey)
		{
			return InKey.CRC;
		}
	};

	virtual void Init();
	void DumpStackTraceToLog(int32 StackIndex);

	friend class FScopeDisableMallocCallstackHandler;

protected:
	/** Malloc we're based on, aka using under the hood */
	FMalloc* UsedMalloc;
	bool Initialized;
	FCriticalSection CriticalSection;
	uint32 DisabledTLS;

	inline void IncDisabled()
	{
		uint64_t DisabledCount = (uint64_t)FPlatformTLS::GetTlsValue(DisabledTLS);
		++DisabledCount;
		FPlatformTLS::SetTlsValue(DisabledTLS, (void*)DisabledCount);
	}

	inline void DecDisabled()
	{
		uint64_t DisabledCount = (uint64_t)FPlatformTLS::GetTlsValue(DisabledTLS);
		--DisabledCount;
		FPlatformTLS::SetTlsValue(DisabledTLS, (void*)DisabledCount);
	}
	virtual bool IsDisabled()
	{
		return FPlatformTLS::GetTlsValue(DisabledTLS) != 0;
	}

	CORE_API virtual void TrackRealloc(void* OldPtr, void* NewPtr, uint32 NewSize, uint32 OldSize, int32 CallStackIndex);
	virtual void TrackMalloc(void* Ptr, uint32 Size, int32 CallStackIndex) = 0;
	virtual void TrackFree(void* Ptr, uint32 OldSize, int32 CallStackIndex) = 0;

	FRWLock RWLock;

	TMap<FCallStackMapKey, int32> CallStackMapKeyToCallStackIndexMap;
	TArray<FCallStackInfo> CallStackInfoArray;

	virtual int32 GetCallStackIndex();
};

extern CORE_API FMallocCallstackHandler* GMallocCallstackHandler;

/**
 * Disables the callstack handler for the current thread
 * Need to do this as we might allocate memory for the allocators tracking data, that can't be tracked!
 */

class FScopeDisableMallocCallstackHandler
{
public:

	FScopeDisableMallocCallstackHandler()
	{
		GMallocCallstackHandler->IncDisabled();
	}

	~FScopeDisableMallocCallstackHandler()
	{
		GMallocCallstackHandler->DecDisabled();
	}
};
