// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/AlignmentTemplates.h"
#include "CoreGlobals.h"

class FOutputDevice;

// Debug memory allocator.
class FMallocDebug : public FMalloc
{
	// Tags.
	enum { MEM_PreTag  = 0xf0ed1cee };
	enum { MEM_PostTag = 0xdeadf00f };
	enum { MEM_Tag     = 0xfe       };
	enum { MEM_WipeTag = 0xcd       };

	// Alignment.
	enum { ALLOCATION_ALIGNMENT = 16 };

	// Number of block sizes to collate (in steps of 4 bytes)
	enum { MEM_SizeMax = 128 };
	enum { MEM_Recent = 5000 };
	enum { MEM_AgeMax = 80 };
	enum { MEM_AgeSlice = 100 };

private:
	// Structure for memory debugging.
	struct FMemDebug
	{
		SIZE_T		Size;
		int32		RefCount;
		int32*		PreTag;
		FMemDebug*	Next;
		FMemDebug**	PrevLink;
	};

	FMemDebug*	GFirstDebug;

	/** Total size of allocations */
	SIZE_T		TotalAllocationSize;
	/** Total size of allocation overhead */
	SIZE_T		TotalWasteSize;

	static constexpr uint32 AllocatorOverhead = sizeof(FMemDebug) + sizeof(FMemDebug*) + sizeof(int32) + ALLOCATION_ALIGNMENT + sizeof(int32);

public:
	// FMalloc interface.
	FMallocDebug()
	:	GFirstDebug( nullptr )
	,	TotalAllocationSize( 0 )
	,	TotalWasteSize( 0 )
	{}

	/** 
	 * Malloc
	 */
	virtual void* Malloc( SIZE_T Size, uint32 Alignment ) override
	{
		check(Alignment <= ALLOCATION_ALIGNMENT && "Alignment currently unsupported in this allocator");
		FMemDebug* Ptr = (FMemDebug*)malloc( AllocatorOverhead + Size );
		check(Ptr);
		uint8* AlignedPtr = Align( (uint8*) Ptr + sizeof(FMemDebug) + sizeof(FMemDebug*) + sizeof(int32), ALLOCATION_ALIGNMENT );
		Ptr->RefCount = 1;

		Ptr->Size = Size;
		Ptr->Next = GFirstDebug;
		Ptr->PrevLink = &GFirstDebug;
		Ptr->PreTag = (int32*) (AlignedPtr - sizeof(int32));
		*Ptr->PreTag = MEM_PreTag;
		*((FMemDebug**)(AlignedPtr - sizeof(int32) - sizeof(FMemDebug*)))	= Ptr;
		*(int32*)(AlignedPtr+Size) = MEM_PostTag;
		FMemory::Memset( AlignedPtr, MEM_Tag, Size );
		if( GFirstDebug )
		{
			check(GIsCriticalError||GFirstDebug->PrevLink==&GFirstDebug);
			GFirstDebug->PrevLink = &Ptr->Next;
		}
		GFirstDebug = Ptr;
		TotalAllocationSize += Size;
		TotalWasteSize += AllocatorOverhead;
		check(!(PTRINT(AlignedPtr) & ((PTRINT)0xf)));
		return AlignedPtr;
	}

	/** 
	 * Realloc
	 */
	virtual void* Realloc( void* InPtr, SIZE_T NewSize, uint32 Alignment ) override
	{
		if( InPtr && NewSize )
		{
			FMemDebug* Ptr = *((FMemDebug**)((uint8*)InPtr - sizeof(int32) - sizeof(FMemDebug*)));
			check(GIsCriticalError||(Ptr->RefCount==1));
			void* Result = Malloc( NewSize, Alignment );
			FMemory::Memcpy( Result, InPtr, FMath::Min<SIZE_T>(Ptr->Size,NewSize) );
			Free( InPtr );
			return Result;
		}
		else if( InPtr == nullptr )
		{
			return Malloc( NewSize, Alignment );
		}
		else
		{
			Free( InPtr );
			return nullptr;
		}
	}

	/** 
	 * Free
	 */
	virtual void Free( void* InPtr ) override
	{
		if( !InPtr )
		{
			return;
		}

		FMemDebug* Ptr = *((FMemDebug**)((uint8*)InPtr - sizeof(int32) - sizeof(FMemDebug*)));

		check(GIsCriticalError||Ptr->RefCount==1);
		check(GIsCriticalError||*Ptr->PreTag==MEM_PreTag);
		check(GIsCriticalError||*(int32*)((uint8*)InPtr+Ptr->Size)==MEM_PostTag);
	
		TotalAllocationSize -= Ptr->Size;
		TotalWasteSize -= AllocatorOverhead;

		FMemory::Memset( InPtr, MEM_WipeTag, Ptr->Size );	
		Ptr->Size = 0;
		Ptr->RefCount = 0;

		check(GIsCriticalError||Ptr->PrevLink);
		check(GIsCriticalError||*Ptr->PrevLink==Ptr);
		*Ptr->PrevLink = Ptr->Next;
		if( Ptr->Next )
		{
			Ptr->Next->PrevLink = Ptr->PrevLink;
		}

		free( Ptr );
	}

	/**
	 * If possible determine the size of the memory allocated at the given address
	 *
 	 * @param Original - Pointer to memory we are checking the size of
	 * @param SizeOut - If possible, this value is set to the size of the passed in pointer
	 * @return true if succeeded
	 */
	virtual bool GetAllocationSize(void *Original, SIZE_T &SizeOut) override
	{
		if( !Original )
		{
			SizeOut = 0;
		}
		else
		{
			const FMemDebug* Ptr = *((FMemDebug**)((uint8*)Original - sizeof(int32) - sizeof(FMemDebug*)));
			SizeOut = Ptr->Size;
		}
		
		return true;
	}

	/**
	 * Dumps details about all allocations to an output device
	 *
	 * @param Ar	[in] Output device
	 */
	virtual void DumpAllocatorStats( FOutputDevice& Ar ) override
	{
		Ar.Logf( TEXT( "Total Allocation Size: %u" ), TotalAllocationSize );
		Ar.Logf( TEXT( "Total Waste Size: %u" ), TotalWasteSize );

		int32 Count = 0;
		int32 Chunks = 0;

		Ar.Logf( TEXT( "" ) );
		Ar.Logf( TEXT( "Unfreed memory:" ) );
		for( FMemDebug* Ptr = GFirstDebug; Ptr; Ptr = Ptr->Next )
		{
			Count += (int32)Ptr->Size;
			Chunks++;
		}

		Ar.Logf( TEXT( "End of list: %i Bytes still allocated" ), Count );
		Ar.Logf( TEXT( "             %i Chunks allocated" ), Chunks );
	}

	/**
	 * Validates the allocator's heap
	 */
	virtual bool ValidateHeap() override
	{
		for( FMemDebug** Link = &GFirstDebug; *Link; Link=&(*Link)->Next )
		{
			check(GIsCriticalError||*(*Link)->PrevLink==*Link);
		}
#if PLATFORM_WINDOWS
		int32 Result = _heapchk();
		check(Result!=_HEAPBADBEGIN);
		check(Result!=_HEAPBADNODE);
		check(Result!=_HEAPBADPTR);
		check(Result!=_HEAPEMPTY);
		check(Result==_HEAPOK);
#endif
		return( true );
	}

	virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) override
	{
		return false;
	}

	virtual const TCHAR* GetDescriptiveName() override { return TEXT("debug"); }

};
