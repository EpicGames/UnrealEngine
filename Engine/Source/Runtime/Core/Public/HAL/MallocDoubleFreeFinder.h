// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/MallocCallstackHandler.h"
#include "HAL/Platform.h"

class FMalloc;
class FOutputDevice;
class UWorld;

class FMallocDoubleFreeFinder final : public FMallocCallstackHandler
{
public:
	CORE_API FMallocDoubleFreeFinder(FMalloc* InMalloc);

#if UE_ALLOW_EXEC_COMMANDS
	/**
	 * Handles any commands passed in on the command line
	 */
	CORE_API virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
#endif

	/**
	 * If you get an allocation/memory error outside of the allocator you can call this directly
	 * It will dump a callstack of the last allocator free most likely to have caused the problem to the log, if you have symbols loaded
	 * Might be useful to pass an access violation ptr to this!
	 */
	CORE_API void TrackSpecial(void* Ptr);

	CORE_API virtual void Init();

	static CORE_API FMalloc* OverrideIfEnabled(FMalloc*InUsedAlloc);

protected:
	CORE_API virtual void TrackMalloc(void* Ptr, uint32 Size, int32 CallStackIndex);
	CORE_API virtual void TrackFree(void* Ptr, uint32 OldSize, int32 CallStackIndex);

	struct TrackedAllocationData
	{
		SIZE_T Size;
		int32 CallStackIndex;
		TrackedAllocationData() :
			Size(0),
			CallStackIndex(-1)
		{
		};
		TrackedAllocationData(SIZE_T InRequestedSize, int32 InCallStackIndex)
		{
			Size = InRequestedSize;
			CallStackIndex = InCallStackIndex;
		};
		~TrackedAllocationData()
		{
			Size = 0;
			CallStackIndex = -1;
		};
	};
	TMap<const void* const, TrackedAllocationData> TrackedCurrentAllocations;	// Pointer as a key to a call stack for all the current allocations we known about
	TMap<const void* const, TrackedAllocationData> TrackedFreeAllocations;		// Pointer as a key to a call stack for all allocations that have been freed
};

extern CORE_API FMallocDoubleFreeFinder* GMallocDoubleFreeFinder;
extern CORE_API bool GMallocDoubleFreeFinderEnabled;
