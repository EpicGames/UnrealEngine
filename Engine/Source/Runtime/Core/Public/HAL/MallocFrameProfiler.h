// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/MallocCallstackHandler.h"
#include "HAL/Platform.h"

class FMalloc;
class FOutputDevice;
class UWorld;

class FMallocFrameProfiler final : public FMallocCallstackHandler
{
public:
	CORE_API FMallocFrameProfiler(FMalloc* InMalloc);

	CORE_API virtual void Init() override;
	
#if UE_ALLOW_EXEC_COMMANDS
	/**
	 * Handles any commands passed in on the command line
	 */
	CORE_API virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
#endif

	/** 
	* Called once per frame, gathers and sets all memory allocator statistics into the corresponding stats. MUST BE THREAD SAFE.
	*/
	CORE_API virtual void UpdateStats() override;

	static CORE_API FMalloc* OverrideIfEnabled(FMalloc*InUsedAlloc);

protected:
	CORE_API virtual bool IsDisabled() override;

	CORE_API virtual void TrackMalloc(void* Ptr, uint32 Size, int32 CallStackIndex);
	CORE_API virtual void TrackFree(void* Ptr, uint32 OldSize, int32 CallStackIndex);
	CORE_API virtual void TrackRealloc(void* OldPtr, void* NewPtr, uint32 NewSize, uint32 OldSize, int32 CallStackIndex);

protected:
	bool bEnabled;
	uint32 FrameCount;
	uint32 EntriesToOutput;

	struct FCallStackStats
	{
		int32 CallStackIndex = 0;
		int32 Mallocs = 0;
		int32 Frees = 0;
		int32 UsageCount = 0;
		int32 UniqueFrames = 0;
		int32 LastFrameSeen = 0;
	};

	TMap<void*, int32> TrackedCurrentAllocations;
	TArray<FCallStackStats> CallStackStatsArray;
};

extern CORE_API FMallocFrameProfiler* GMallocFrameProfiler;
extern CORE_API bool GMallocFrameProfilerEnabled;
