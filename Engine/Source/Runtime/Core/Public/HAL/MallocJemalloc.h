// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "HAL/MemoryBase.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Trace/Detail/Channel.h"

class FOutputDevice;

// Only use for supported platforms
#if PLATFORM_SUPPORTS_JEMALLOC

#define MEM_TIME(st)


//
// Jason Evans' malloc (default in FreeBSD, NetBSD, used in Firefox, Facebook servers) 
// http://www.canonware.com/jemalloc/
//
class FMallocJemalloc
	: public FMalloc
{
public:

	/**
	 * Default constructor.
	 */
	FMallocJemalloc()
		: MemTime(0.0)
	{ }
	
public:

	// FMalloc interface.

	virtual void* Malloc( SIZE_T Size, uint32 Alignment ) override;
	virtual void* Realloc( void* Ptr, SIZE_T NewSize, uint32 Alignment ) override;
	virtual void Free( void* Ptr ) override;
	virtual void DumpAllocatorStats( FOutputDevice& Ar ) override;
	virtual bool GetAllocationSize(void *Original, SIZE_T &SizeOut) override;
	virtual bool IsInternallyThreadSafe() const override {  return true; }
	virtual const TCHAR* GetDescriptiveName() override { return TEXT("jemalloc"); }

protected:

	void OutOfMemory( )
	{
		UE_LOGF(LogHAL, Fatal, "%ls", TEXT("Ran out of virtual memory. To prevent this condition, you must free up more space on your primary hard disk.") );
	}

private:

	/** Time spent on memory tasks, in seconds */
	double MemTime;
};


#endif // PLATFORM_SUPPORTS_JEMALLOC
