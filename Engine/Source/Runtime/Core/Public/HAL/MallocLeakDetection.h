// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MallocLeakDetection.h: Helper class to track memory allocations
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/MemoryBase.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/UnrealMemory.h"
#include "Hash/CityHash.h"
#include "Misc/Crc.h"
#include "Misc/ScopeLock.h"

#ifndef MALLOC_LEAKDETECTION
	#define MALLOC_LEAKDETECTION 0
#endif

/**
 *	Options that can be supplied when calling FMallocLeakDetection::DumpOpenCallstacks
 */
struct FMallocLeakReportOptions
{
	enum class ESortOption
	{
		SortSize,
		SortRate,
		SortHash
	};

	FMallocLeakReportOptions()
	{
		FMemory::Memzero(this, sizeof(FMallocLeakReportOptions));
	}

	/** If >0 only report allocations greater than this size */
	uint32			SizeFilter;

	/** If >0 only report allocations at a greater bytes/frame than this */
	float			RateFilter;

	/** Restrict report to allocations that have no history of being deleted */
	bool			OnlyNonDeleters;

	/** Only show allocations after this frame */
	uint32			FrameStart;

	/** Only show allocations from before this frame */
	uint32			FrameEnd;

	/** Sort allocations by this (default - size) */
	ESortOption		SortBy;

	/** Output device used for writing reports. If null, a memreport file will be written */
	FOutputDevice*	OutputDevice;
};

#if MALLOC_LEAKDETECTION

/**
 * Maintains a list of all pointers to currently allocated memory.
 */
class FMallocLeakDetection
{
	using CallstackHashType = uint64;

	struct FCallstackTrack
	{
		FCallstackTrack()
		{
			FMemory::Memzero(this, sizeof(FCallstackTrack));
		}
		static constexpr int32 Depth = 32;
		uint64 Callstack[Depth];
		uint32 FirstFrame;
		uint32 LastFrame;
		uint64 Size;
		CallstackHashType CachedHash;
		uint32 Count;

		// least square line fit stuff
		uint32 NumCheckPoints;
		float SumOfFramesNumbers;
		float SumOfFramesNumbersSquared;
		float SumOfMemory;
		float SumOfMemoryTimesFrameNumber;

		// least square line results
		float Baseline;
		float BytesPerFrame;

		bool operator==(const FCallstackTrack& Other) const
		{
			bool bEqual = true;
			for (int32 i = 0; i < Depth; ++i)
			{
				if (Callstack[i] != Other.Callstack[i])
				{
					bEqual = false;
					break;
				}
			}
			return bEqual;
		}

		bool operator!=(const FCallstackTrack& Other) const
		{
			return !(*this == Other);
		}

		void GetLinearFit();
		
		CallstackHashType GetHash() 
		{
			const char* Data = reinterpret_cast<const char*>(Callstack);
			CachedHash = CityHash64(Data, sizeof(Callstack));
			return CachedHash;
		}
	};

private:

	CORE_API FMallocLeakDetection();
	CORE_API ~FMallocLeakDetection();

	/** Track a callstack */

	/** Stop tracking a callstack */
	CORE_API void AddCallstack(FCallstackTrack& Callstack);
	CORE_API void RemoveCallstack(FCallstackTrack& Callstack);

	/** List of all currently allocated pointers */
	TMap<void*, FCallstackTrack> OpenPointers;

	/** List of all unique callstacks with allocated memory */
	TMap<CallstackHashType, FCallstackTrack> UniqueCallstacks;

	/** Set of callstacks that are known to delete memory (not reset on ClearData()) */
	TSet<CallstackHashType>	KnownDeleters;

	/** Set of callstacks that are known to resize memory (not reset on ClearData()) */
	TSet<CallstackHashType>	KnownTrimmers;

	/** Contexts that are associated with allocations */
	TMap<void*, FString>		PointerContexts;

	/** Stack of contexts */
	struct FContextString { TCHAR Buffer[64]; };
	TArray<FContextString>	Contexts;
		
	/** Critical section for mutating internal data */
	FCriticalSection AllocatedPointersCritical;	

	/** Set during mutating operations to prevent internal allocations from recursing */
	bool	bRecursive;

	/** Is allocation capture enabled? */
	bool	bCaptureAllocs;

	/** Minimal size to capture? */
	int32	MinAllocationSize;

	/** Size of all tracked callstacks */
	SIZE_T	TotalTracked;

	/** How long since we compacted things? */
	int32	AllocsWithoutCompact;

public:	

	static CORE_API FMallocLeakDetection& Get();
	static CORE_API void HandleMallocLeakCommand(const TArray< FString >& Args);

	/** Enable/disable collection of allocation with an optional filter on allocation size */
	CORE_API void SetAllocationCollection(bool bEnabled, int32 Size = 0);

	/** Returns state of allocation collection */
	bool IsAllocationCollectionEnabled(void) const { return bCaptureAllocs; }

	/** Clear currently accumulated data */
	CORE_API void ClearData();

	/** Dumps currently open callstacks */
	CORE_API int32 DumpOpenCallstacks(const TCHAR* FileName, const FMallocLeakReportOptions& Options = FMallocLeakReportOptions());

	/** Perform a linear fit checkpoint of all open callstacks */
	CORE_API void CheckpointLinearFit();

	/** Handles new allocated pointer */
	CORE_API void Malloc(void* Ptr, SIZE_T Size);

	/** Handles reallocation */
	CORE_API void Realloc(void* OldPtr, SIZE_T OldSize, void* NewPtr, SIZE_T NewSize);

	/** Removes allocated pointer from list */
	CORE_API void Free(void* Ptr);	

	/** Disabled allocation tracking for this thread, @see MALLOCLEAK_IGNORE_SCOPE and FMallocLeakDetectionIgnoreScope. */
	CORE_API void SetDisabledForThisThread(const bool Disabled);

	/** Returns true of allocation tracking for this thread is  */
	CORE_API bool IsDisabledForThisThread() const;

	/** Pushes context that will be associated with allocations. All open contexts will be displayed alongside
	callstacks in a report.  */
	void PushContext(const FString& Context)
	{
		this->PushContext(*Context);
	}
	CORE_API void PushContext(const TCHAR* Context);

	/** Pops a context from the above */
	CORE_API void PopContext();

	/** Returns */
	CORE_API void GetOpenCallstacks(TArray<CallstackHashType>& OutCallstacks, SIZE_T& OutTotalSize, const FMallocLeakReportOptions& Options = FMallocLeakReportOptions());
};

/**
 *	Helper class that can be used to ignore allocations from a specific scope for leak detection.
 *	Use this carefully and only if you know that a portion of code is throwing up either false
 *	positives or can be ignored. (e.g., one example is the FName table which never shrinks
 *	and eventually reaches a max that is relatively inconsequential).
 */
class FMallocLeakDetectionIgnoreScope
{
public:
	FMallocLeakDetectionIgnoreScope()
	{
		FMallocLeakDetection::Get().SetDisabledForThisThread(true);
	}

	~FMallocLeakDetectionIgnoreScope()
	{
		FMallocLeakDetection::Get().SetDisabledForThisThread(false);
	}

	// Non-copyable
	FMallocLeakDetectionIgnoreScope(const FMallocLeakDetectionIgnoreScope&) = delete;
	FMallocLeakDetectionIgnoreScope& operator=(const FMallocLeakDetectionIgnoreScope&) = delete;
};

class FMallocLeakScopedContext
{
public:
	template <typename ArgType>
	explicit FMallocLeakScopedContext(ArgType&& Context)
	{
		FMallocLeakDetection::Get().PushContext(Forward<ArgType>(Context));
	}

	~FMallocLeakScopedContext()
	{
		FMallocLeakDetection::Get().PopContext();
	}

	// Non-copyable
	FMallocLeakScopedContext(const FMallocLeakScopedContext&) = delete;
	FMallocLeakScopedContext& operator=(const FMallocLeakScopedContext&) = delete;
};

#define MALLOCLEAK_IGNORE_SCOPE() \
	FMallocLeakDetectionIgnoreScope ANONYMOUS_VARIABLE(DetectionShouldIgnoreScope)

#define MALLOCLEAK_SCOPED_CONTEXT(Context) \
	FMallocLeakScopedContext ANONYMOUS_VARIABLE(ScopedContext)(Context)

#else // MALLOC_LEAKDETECTION 0

#define MALLOCLEAK_IGNORE_SCOPE()
#define MALLOCLEAK_SCOPED_CONTEXT(Context)

#endif // MALLOC_LEAKDETECTION

