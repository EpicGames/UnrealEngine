// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformMemory.h"
#include "HAL/MemoryBase.h"

#if !defined(PLATFORM_BUILDS_LIBPAS)
#	define PLATFORM_BUILDS_LIBPAS 0
#endif

#define LIBPASMALLOC_ENABLED PLATFORM_BUILDS_LIBPAS

#if LIBPASMALLOC_ENABLED

/**
 * The malloc from Phil's Awesome System (libpas).
 */
class FMallocLibpas final
	: public FMalloc
{
public:
	FMallocLibpas();
	virtual void* Malloc(SIZE_T Size, uint32 Alignment) override;
	virtual void* TryMalloc(SIZE_T Size, uint32 Alignment) override;
	virtual void* Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override;
	virtual void* TryRealloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override;
	virtual void Free(void* Ptr) override;
	virtual bool GetAllocationSize(void *Original, SIZE_T &SizeOut) override;
	virtual void Trim(bool bTrimThreadCaches) override;

	virtual bool IsInternallyThreadSafe() const override
	{ 
		return true;
	}

	virtual const TCHAR* GetDescriptiveName( ) override
	{
		return TEXT("Libpas");
	}

	virtual void OnPreFork() override;
	virtual void OnPostFork() override;
};

#endif // LIBPASMALLOC_ENABLED
