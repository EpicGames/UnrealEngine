// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformMemory.h"
#include "HAL/MemoryBase.h"

#if !defined(PLATFORM_BUILDS_MIMALLOC)
#	define PLATFORM_BUILDS_MIMALLOC 0
#endif

#if !defined(PLATFORM_SUPPORTS_MIMALLOC)
#	define PLATFORM_SUPPORTS_MIMALLOC 0
#endif

#define MIMALLOC_ENABLED PLATFORM_SUPPORTS_MIMALLOC && MIMALLOC_ALLOCATOR_ALLOWED && PLATFORM_BUILDS_MIMALLOC

#if MIMALLOC_ENABLED

/**
 * MiMalloc 64-bit scalable memory allocator.
 */
class FMallocMimalloc
	: public FMalloc
{
public:
	// FMalloc interface.
	FMallocMimalloc();
	virtual void* Malloc(SIZE_T Size, uint32 Alignment) override;
	virtual void* TryMalloc(SIZE_T Size, uint32 Alignment) override;
	virtual void* Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override;
	virtual void* TryRealloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override;
	virtual void Free(void* Ptr) override;
	virtual void* MallocZeroed(SIZE_T Count, uint32 Alignment) override;
	virtual void* TryMallocZeroed(SIZE_T Count, uint32 Alignment) override;
	virtual bool GetAllocationSize(void *Original, SIZE_T &SizeOut) override;
	virtual void Trim(bool bTrimThreadCaches) override;

	virtual bool IsInternallyThreadSafe() const override
	{ 
		return true;
	}

	virtual const TCHAR* GetDescriptiveName( ) override
	{
		return TEXT("Mimalloc");
	}

protected:

	void OutOfMemory( uint64 Size, uint32 Alignment )
	{
		// this is expected not to return
		FPlatformMemory::OnOutOfMemory(Size, Alignment);
	}
};

#endif
