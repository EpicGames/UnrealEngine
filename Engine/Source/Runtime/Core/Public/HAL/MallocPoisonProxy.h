// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/MemoryBase.h"
#include "HAL/UnrealMemory.h"

/** Governs when malloc that poisons the allocations is enabled. */
#if !defined(UE_USE_MALLOC_FILL_BYTES)
	#define UE_USE_MALLOC_FILL_BYTES ((UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT) && !WITH_EDITORONLY_DATA && !PLATFORM_USES_FIXED_GMalloc_CLASS && !USING_ADDRESS_SANITISER)
#endif // !defined(UE_USE_MALLOC_FILL_BYTES)

/** Value that a freed memory block will be filled with when UE_USE_MALLOC_FILL_BYTES is defined. */
#define UE_DEBUG_FILL_FREED (0xdd)

/** Value that a new memory block will be filled with when UE_USE_MALLOC_FILL_BYTES is defined. */
#define UE_DEBUG_FILL_NEW (0xcd)

/**
 * FMalloc proxy that poisons new and freed allocations, helping to catch code that relies on uninitialized or freed memory.
 */
class FMallocPoisonProxy : public FMalloc
{
private:
	/** Malloc we're based on, aka using under the hood */
	FMalloc* UsedMalloc;

public:
	// FMalloc interface begin
	explicit FMallocPoisonProxy(FMalloc* InMalloc)
		: UsedMalloc(InMalloc)
	{
		checkf(UsedMalloc, TEXT("FMallocPoisonProxy is used without a valid malloc!"));
	}

	virtual void InitializeStatsMetadata() override
	{
		UsedMalloc->InitializeStatsMetadata();
	}

	virtual void* Malloc(SIZE_T Size, uint32 Alignment) override
	{
		void* Result = UsedMalloc->Malloc(Size, Alignment);
		if (LIKELY(Result != nullptr && Size > 0))
		{
			FMemory::Memset(Result, UE_DEBUG_FILL_NEW, Size);
		}
		return Result;
	}

	virtual void* MallocZeroed(SIZE_T Size, uint32 Alignment) override
	{
		void* Result = UsedMalloc->Malloc(Size, Alignment);
		if (LIKELY(Result != nullptr && Size > 0))
		{
			FMemory::Memset(Result, 0, Size);
		}
		return Result;
	}

	virtual void* Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override
	{
		// NOTE: case when Realloc returns completely new pointer is not handled properly (we would like to have previous memory poisoned completely).
		// This can be done by avoiding calling Realloc() of the nested allocator and Malloc()/Free()'ing directly, but this is deemed unacceptable
		// from a performance/fragmentation standpoint.
		SIZE_T OldSize = 0;
		if (Ptr != nullptr && GetAllocationSize(Ptr, OldSize) && OldSize > 0 && OldSize > NewSize)
		{
			FMemory::Memset(static_cast<uint8*>(Ptr) + NewSize, UE_DEBUG_FILL_FREED, OldSize - NewSize);
		}

		void* Result = UsedMalloc->Realloc(Ptr, NewSize, Alignment);

		if (Result != nullptr && OldSize > 0 && OldSize < NewSize)
		{
			FMemory::Memset(static_cast<uint8*>(Result) + OldSize, UE_DEBUG_FILL_NEW, NewSize - OldSize);
		}

		return Result;
	}

	virtual void Free(void* Ptr) override
	{
		if (LIKELY(Ptr))
		{
			SIZE_T AllocSize;
			if (LIKELY(GetAllocationSize(Ptr, AllocSize) && AllocSize > 0))
			{
				FMemory::Memset(Ptr, UE_DEBUG_FILL_FREED, AllocSize);
			}
			UsedMalloc->Free(Ptr);
		}
	}

	virtual SIZE_T QuantizeSize(SIZE_T Count, uint32 Alignment) override
	{
		return UsedMalloc->QuantizeSize(Count, Alignment);
	}

	virtual void UpdateStats() override
	{
		UsedMalloc->UpdateStats();
	}

	virtual void GetAllocatorStats( FGenericMemoryStats& out_Stats ) override
	{
		UsedMalloc->GetAllocatorStats( out_Stats );
	}

	virtual void DumpAllocatorStats( class FOutputDevice& Ar ) override
	{
		UsedMalloc->DumpAllocatorStats( Ar );
	}

	virtual bool IsInternallyThreadSafe() const override
	{
		return UsedMalloc->IsInternallyThreadSafe();
	}

	virtual bool ValidateHeap() override
	{
		return UsedMalloc->ValidateHeap();
	}

#if UE_ALLOW_EXEC_COMMANDS
	virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) override
	{
		return UsedMalloc->Exec(InWorld, Cmd, Ar);
	}
#endif // UE_ALLOW_EXEC_COMMANDS

	virtual bool GetAllocationSize(void *Original, SIZE_T &SizeOut) override
	{
		return UsedMalloc->GetAllocationSize(Original,SizeOut);
	}

	virtual const TCHAR* GetDescriptiveName() override
	{ 
		return UsedMalloc->GetDescriptiveName(); 
	}

	virtual void Trim(bool bTrimThreadCaches) override
	{
		UsedMalloc->Trim(bTrimThreadCaches);
	}

	virtual void SetupTLSCachesOnCurrentThread() override
	{
		UsedMalloc->SetupTLSCachesOnCurrentThread();
	}

	virtual void MarkTLSCachesAsUsedOnCurrentThread() override
	{
		UsedMalloc->MarkTLSCachesAsUsedOnCurrentThread();
	}

	virtual void MarkTLSCachesAsUnusedOnCurrentThread() override
	{
		UsedMalloc->MarkTLSCachesAsUnusedOnCurrentThread();
	}

	virtual void ClearAndDisableTLSCachesOnCurrentThread() override
	{
		UsedMalloc->ClearAndDisableTLSCachesOnCurrentThread();
	}

	virtual void OnMallocInitialized() override
	{
		UsedMalloc->OnMallocInitialized();
	}

	virtual void OnPreFork() override
	{
		UsedMalloc->OnPreFork();
	}


	virtual void OnPostFork() override
	{
		UsedMalloc->OnPostFork();
	}

	// FMalloc interface end
};
