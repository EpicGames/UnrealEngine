// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/MemoryBase.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ScopeLock.h"

class FOutputDevice;
class UWorld;
struct FGenericMemoryStats;

#if !defined(UE_USE_MALLOC_REPLAY_PROXY)
	// it is always enabled on Linux, but not always added to the malloc stack
	#define UE_USE_MALLOC_REPLAY_PROXY						(PLATFORM_UNIX && !UE_BUILD_SHIPPING)
#endif // !defined(UE_USE_MALLOC_REPLAY_PROXY)

#if UE_USE_MALLOC_REPLAY_PROXY

/**
 * This FMalloc proxy is used as a lighweight way to dump memory allocation for later replaying
 * (and thus testing different malloc implementations)
 */
class FMallocReplayProxy : public FMalloc
{
private:
	/** Malloc we're based on, aka using under the hood */
	FMalloc* UsedMalloc;

private:

	enum
	{
		HistoryCacheSize		=	16384
	};

	/** describes a single operation entry */
	struct FHistoryEntry
	{
		/** We don't need to compare the operations, so we can store this as string */
		const char*		Operation;
		/** Pointer to memory chunk as returned. */
		void *			PointerOut;
		/** Pointer to memory chunk as passed in. */
		void *			PointerIn;
		/** Size as passed in - only valid for malloc/realloc. */
		SIZE_T			Size;
		/** Alignment as passed in - only valid for malloc/realloc. */
		uint32			Alignment;
	};

	/** Size of history not yet dumped to disk */
	FHistoryEntry		HistoryCache[HistoryCacheSize];

	/** Current entry in history */
	int32				CurrentCacheIdx;

	/** Global operation number (to aid using dump file) */
	uint64				OperationNumber;

	/** Guards access to history*/
	FCriticalSection	HistoryLock;

	/** Regular file where we save the history to */
	FILE*				HistoryFile;

	/** Adds operation to history*/
	void AddToHistory(const char *Op, void * PtrOut, void * PtrIn, SIZE_T Size, SIZE_T Alignment)
	{
		FScopeLock Lock(&HistoryLock);

		HistoryCache[CurrentCacheIdx].Operation = Op;
		HistoryCache[CurrentCacheIdx].PointerOut = PtrOut;
		HistoryCache[CurrentCacheIdx].PointerIn = PtrIn;
		HistoryCache[CurrentCacheIdx].Size = Size;
		HistoryCache[CurrentCacheIdx].Alignment = Alignment;

		++CurrentCacheIdx;
		if (CurrentCacheIdx > HistoryCacheSize - 1)
		{
			DumpHistoryToDisk();
		}
	}

	/** Expected to be called while history lock is taken. */
	void DumpHistoryToDisk();

public:
	// FMalloc interface begin
	explicit FMallocReplayProxy(FMalloc* InMalloc);

	virtual ~FMallocReplayProxy();

	virtual void InitializeStatsMetadata() override;

	virtual void* Malloc(SIZE_T Size, uint32 Alignment) override;

	virtual void* Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override;

	virtual void Free(void* Ptr) override;

	virtual SIZE_T QuantizeSize(SIZE_T Count, uint32 Alignment) override;

	virtual void UpdateStats() override;

	virtual void GetAllocatorStats(FGenericMemoryStats& out_Stats) override;

	virtual void DumpAllocatorStats(class FOutputDevice& Ar) override;

	virtual bool IsInternallyThreadSafe() const override;

	virtual bool ValidateHeap() override;

	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

	virtual bool GetAllocationSize(void *Original, SIZE_T &SizeOut) override;

	virtual const TCHAR* GetDescriptiveName() override;

	virtual void Trim(bool bTrimThreadCaches) override;

	virtual void SetupTLSCachesOnCurrentThread() override;

	virtual void ClearAndDisableTLSCachesOnCurrentThread() override;

	// FMalloc interface end

	// called by destructor or otherwise, idempotent
	void CloseHistory();
};

#endif // UE_USE_MALLOC_REPLAY_PROXY
