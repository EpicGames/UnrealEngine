// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/MemoryBase.h"

class FOutputDevice;
class UWorld;

/**
 * Stomp memory allocator support should be enabled in Core.Build.cs.
 * Run-time validation should be enabled using '-stompmalloc' command line argument.
 */

#if WITH_MALLOC_STOMP

#if PLATFORM_WINDOWS
	#include "Windows/WindowsHWrapper.h"
#endif

/**
 * Stomp memory allocator. It helps find the following errors:
 * - Read or writes off the end of an allocation.
 * - Read or writes off the beginning of an allocation.
 * - Read or writes after freeing an allocation. 
 */
class FMallocStomp final : public FMalloc
{
	struct FAllocationData;

	const SIZE_T PageSize;

	/** If it is set to true, instead of focusing on overruns the allocator will focus on underruns. */
	const bool bUseUnderrunMode;

	static constexpr SIZE_T VirtualAddressBlockSize = 1 * 1024 * 1024 * 1024; // 1 GB blocks

public:
	// FMalloc interface.
	explicit FMallocStomp(const bool InUseUnderrunMode = false);

	/**
	 * Allocates a block of a given number of bytes of memory with the required alignment.
	 * In the process it allocates as many pages as necessary plus one that will be protected
	 * making it unaccessible and causing an exception. The actual allocation will be pushed
	 * to the end of the last valid unprotected page. To deal with underrun errors a sentinel
	 * is added right before the allocation in page which is checked on free.
	 *
	 * @param Size Size in bytes of the memory block to allocate.
	 * @param Alignment Alignment in bytes of the memory block to allocate.
	 * @return A pointer to the beginning of the memory block.
	 */
	virtual void* Malloc(SIZE_T Size, uint32 Alignment) override;

	virtual void* TryMalloc(SIZE_T Size, uint32 Alignment) override;

	/**
	 * Changes the size of the memory block pointed to by OldPtr.
	 * The function may move the memory block to a new location.
	 *
	 * @param OldPtr Pointer to a memory block previously allocated with Malloc. 
	 * @param NewSize New size in bytes for the memory block.
	 * @param Alignment Alignment in bytes for the reallocation.
	 * @return A pointer to the reallocated memory block, which may be either the same as ptr or a new location.
	 */
	virtual void* Realloc(void* InPtr, SIZE_T NewSize, uint32 Alignment) override;

	virtual void* TryRealloc(void* InPtr, SIZE_T NewSize, uint32 Alignment) override;

	/**
	 * Frees a memory allocation and verifies the sentinel in the process.
	 *
	 * @param InPtr Pointer of the data to free.
	 */
	virtual void Free(void* InPtr) override;

	/**
	 * If possible determine the size of the memory allocated at the given address.
	 * This will included all the pages that were allocated so it will be far more
	 * than what's set on the FAllocationData.
	 *
	 * @param Original - Pointer to memory we are checking the size of
	 * @param SizeOut - If possible, this value is set to the size of the passed in pointer
	 * @return true if succeeded
	 */
	virtual bool GetAllocationSize(void *Original, SIZE_T &SizeOut) override;

	/**
	 * Dumps details about all allocations to an output device
	 *
	 * @param Ar	[in] Output device
	 */
	virtual void DumpAllocatorStats( FOutputDevice& Ar ) override
	{
		// No meaningful stats to dump.
	}

	/**
	 * Validates the allocator's heap
	 */
	virtual bool ValidateHeap() override
	{
		// Nothing to do here since validation happens as data is accessed
		// through page protection, and on each free checking the sentinel.
		return true;
	}

#if UE_ALLOW_EXEC_COMMANDS
	virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) override
	{
		return false;
	}
#endif

	virtual const TCHAR* GetDescriptiveName() override
	{
		return TEXT( "Stomp" );
	}

	virtual bool IsInternallyThreadSafe() const override
	{
		// Unix and Mac uses mmap which is thread-safe
		// Windows uses virtual alloc which is thread-safe
		// Other platforms using FPlatformMemory::BinnedAllocFromOS are all assumed to have thread-safe implementation already
		// VirtualAddressCursor and VirtualAddressMax are thread-locals so each thread has its virtual reserved block
		return true;
	}
};

#endif // WITH_MALLOC_STOMP
