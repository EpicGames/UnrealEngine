// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "HAL/MemoryBase.h"
#include "HAL/PlatformTLS.h"

#ifndef WITH_MALLOC_STOMP2
#define WITH_MALLOC_STOMP2 WITH_EDITOR && PLATFORM_WINDOWS
#endif
#if WITH_MALLOC_STOMP2
#include <atomic>
/**
 * Stomp memory allocator support should be enabled in Core.Build.cs.
 * Run-time validation should be enabled using '-stomp2malloc' command line argument.
 */

#if PLATFORM_MICROSOFT
	#include "Microsoft/WindowsHWrapper.h"
#endif

/**
 * Stomp memory allocator. It helps find the following errors:
 * - Read or writes off the end of an allocation.
 * - Read or writes off the beginning of an allocation.
 * - Read or writes after freeing an allocation. 
 */
class FMallocStomp2 final : public FMalloc
{
private:
	/** Expected value to be found in the sentinel. */
	static constexpr SIZE_T SentinelExpectedValue = 0xdeadbeefdeadbeef;

	const SIZE_T PageSize;

	struct FAllocationData
	{
		/** Pointer to the full allocation. Needed so the OS knows what to free. */
		void	*FullAllocationPointer;
		/** Full size of the allocation including the extra page. */
		SIZE_T	FullSize;
		/** Size of the allocation requested. */
		SIZE_T	Size;
		/** Sentinel used to check for underrun. */
		SIZE_T	Sentinel;
	};

public:
	struct FAddressSpaceStompDataBlockRangeEntry
	{
		UPTRINT	StartAddress;
		UPTRINT	EndAddress;
		int		StartIndex;
		int		EndIndex;
	};



	struct FAddressSpaceStompDataBlock
	{
		static constexpr SIZE_T VirtualAddressSpaceToReserve = 1024LL * 1024LL * 1024LL * 64LL; // 64 GB blocks, we try to reserve that much, but will half it and retry
		std::atomic<UPTRINT>	CurrentAddress;
		UPTRINT	StartAddress;
		UPTRINT	EndAddress;
		SIZE_T	Size;

		bool Init()
		{
			SIZE_T SizeToTryAndReserve = VirtualAddressSpaceToReserve;
			// if we can't reserve the full size, half the size and try again as long as we are over 1GB
			while (SizeToTryAndReserve >= 1024LL * 1024LL * 1024LL)
			{
				StartAddress = (UPTRINT) ::VirtualAlloc(nullptr, SizeToTryAndReserve, MEM_RESERVE, PAGE_READWRITE);
				if (StartAddress)
				{
					EndAddress = StartAddress + SizeToTryAndReserve;
					Size = SizeToTryAndReserve;
					CurrentAddress = StartAddress;
					return true;
				}
				SizeToTryAndReserve /= 2;
			}
			return false;
		}

		void* Malloc(SIZE_T InSizeToAllocate, SIZE_T InPageSize, bool InUnderrun = false)
		{
			UPTRINT Ret = CurrentAddress.fetch_add(InSizeToAllocate);
			if (Ret + InSizeToAllocate > EndAddress)
			{
				return nullptr;
			}
			if (!InUnderrun)
			{
				// leave the last page without backing store
				Ret = (UPTRINT) ::VirtualAlloc((void*)Ret, InSizeToAllocate - InPageSize, MEM_COMMIT, PAGE_READWRITE);
			}
			else
			{
				// leave the first page without backing store
				Ret = (UPTRINT) ::VirtualAlloc((void*)(Ret+InPageSize), InSizeToAllocate, MEM_COMMIT, PAGE_READWRITE);
			}
			return (void*)Ret;
		}
		
		void Free(void* InPtr, SIZE_T InSize)
		{
#if PLATFORM_WINDOWS && USING_CODE_ANALYSIS
	MSVC_PRAGMA(warning(push))
	MSVC_PRAGMA(warning(disable : 6250)) // Suppress C6250, as virtual address space is "leaked" by design.
#endif
			// Unmap physical memory, but keep virtual address range reserved to catch use-after-free errors.
			::VirtualFree(InPtr, InSize, MEM_DECOMMIT);

#if PLATFORM_WINDOWS && USING_CODE_ANALYSIS
	MSVC_PRAGMA(warning(pop))
#endif
		}
	};


private:
	static constexpr SIZE_T TargetAddressSpaceToReserve = 1024LL * 1024LL * 1024LL * 1024LL * 32LL; // how much address space we will try and reserve, 32 TB


	static constexpr SIZE_T MaxAddressSpaceStompDataBlocks = 4096; // this * VirtualAddressSpaceToReserve is how much address space we will try and allocate

	std::atomic<int64> CurrentAddressSpaceStompDataBlockIndex;
	FAddressSpaceStompDataBlock		AddressSpaceStompDataBlocks[MaxAddressSpaceStompDataBlocks];

	int NumberOfRangeEntries;
	FAddressSpaceStompDataBlockRangeEntry AddressSpaceStompDataBlocksRangeEntries[MaxAddressSpaceStompDataBlocks];

	FMalloc* UsedMalloc;

	int IsPartOf(void* InPtr);
	int FindAddressSpaceStompDataBlockIndex(void* InPtr, int Index);


	/** If it is set to true, instead of focusing on overruns the allocator will focus on underruns. */
	const bool bUseUnderrunMode;


public:
	// FMalloc interface.
	FMallocStomp2(FMalloc* InMalloc, const bool InUseUnderrunMode = false);

	/**
	 * Allocates a block of a given number of bytes of memory with the required alignment.
	 * In the process it allocates as many pages as necessary plus one that will be protected
	 * making it unaccessible and causing an exception. The actual allocation will be pushed
	 * to the end of the last valid unprotected page. To deal with underrun errors a sentinel
	 * is added right before the allocation in page which is checked on free.
	 *
	 * @param Size Size in bytes of the memory block to allocate.
	 * @param Alignment Alignment in bytes of the memory block to allocate.
	 * @return A pointer to the beginning of the memory block.
	 */
	virtual void* Malloc(SIZE_T Size, uint32 Alignment) override;

	virtual void* TryMalloc(SIZE_T Size, uint32 Alignment) override;

	/**
	 * Changes the size of the memory block pointed to by OldPtr.
	 * The function may move the memory block to a new location.
	 *
	 * @param OldPtr Pointer to a memory block previously allocated with Malloc. 
	 * @param NewSize New size in bytes for the memory block.
	 * @param Alignment Alignment in bytes for the reallocation.
	 * @return A pointer to the reallocated memory block, which may be either the same as ptr or a new location.
	 */
	virtual void* Realloc(void* InPtr, SIZE_T NewSize, uint32 Alignment) override;

	virtual void* TryRealloc(void* InPtr, SIZE_T NewSize, uint32 Alignment) override;

	/**
	 * Frees a memory allocation and verifies the sentinel in the process.
	 *
	 * @param InPtr Pointer of the data to free.
	 */
	virtual void Free(void* InPtr) override;

	/**
	 * If possible determine the size of the memory allocated at the given address.
	 * This will included all the pages that were allocated so it will be far more
	 * than what's set on the FAllocationData.
	 *
	 * @param Original - Pointer to memory we are checking the size of
	 * @param SizeOut - If possible, this value is set to the size of the passed in pointer
	 * @return true if succeeded
	 */
	virtual bool GetAllocationSize(void *Original, SIZE_T &SizeOut) override;

	/**
	 * Dumps details about all allocations to an output device
	 *
	 * @param Ar	[in] Output device
	 */
	virtual void DumpAllocatorStats( FOutputDevice& Ar ) override
	{
		// No meaningful stats to dump.
	}

	/**
	 * Validates the allocator's heap
	 */
	virtual bool ValidateHeap() override
	{
		// Nothing to do here since validation happens as data is accessed
		// through page protection, and on each free checking the sentinel.
		return true;
	}

	virtual const TCHAR* GetDescriptiveName() override
	{
		return TEXT( "Stomp2" );
	}

	virtual bool IsInternallyThreadSafe() const override
	{
		// Stomp allocator 2 is thread-safe
		return true;
	}

	static FMalloc* OverrideIfEnabled(FMalloc* InUsedAlloc);

	void Init();


	static uint32 DisableTlsSlot;


	/** We don't handle allocations smaller than this size. */
	static SIZE_T MinSize;
	/** We don't handle allocations larger than this size. */
	static SIZE_T MaxSize;
};


/**
 * Implements a scoped disabling of the stomp allocator 2
 *
 * We use this when we need to make a memory allocation that we don't want/need to track
 * 
 */
class FScopeDisableMallocStomp2
{
public:

	/**
	 * Constructor that increments the TLS disables stomallocator2 counter
	 */
	FScopeDisableMallocStomp2()
	{
		uint64 DisableCounter = (uint64)FPlatformTLS::GetTlsValue(FMallocStomp2::DisableTlsSlot);
		++DisableCounter;
		FPlatformTLS::SetTlsValue(FMallocStomp2::DisableTlsSlot, (void*)DisableCounter);
	}

	/** Destructor that performs a decrement on the TLS disable stomallocator2 counter */
	~FScopeDisableMallocStomp2()
	{
		uint64 DisableCounter = (uint64)FPlatformTLS::GetTlsValue(FMallocStomp2::DisableTlsSlot);
		--DisableCounter;
		FPlatformTLS::SetTlsValue(FMallocStomp2::DisableTlsSlot, (void*)DisableCounter);
	}
};

extern CORE_API FMallocStomp2* GMallocStomp2;
extern CORE_API bool GMallocStomp2Enabled;

#endif	//WITH_MALLOC_STOMP2