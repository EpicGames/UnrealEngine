// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/MemoryBase.h"
#include "HAL/PlatformMemory.h"

#define TBBMALLOC_ENABLED PLATFORM_SUPPORTS_TBB && TBB_ALLOCATOR_ALLOWED

#if TBBMALLOC_ENABLED

/**
 * TBB 64-bit scalable memory allocator.
 */
class FMallocTBB final
	: public FMalloc
{
public:
	// FMalloc interface.
	
	virtual void* Malloc(SIZE_T Size, uint32 Alignment) override;
	virtual void* TryMalloc(SIZE_T Size, uint32 Alignment) override;
	virtual void* Realloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override;
	virtual void* TryRealloc(void* Ptr, SIZE_T NewSize, uint32 Alignment) override;
	virtual void Free( void* Ptr ) override;
	virtual bool GetAllocationSize( void *Original, SIZE_T &SizeOut ) override;
	virtual void Trim(bool bTrimThreadCaches) override;

	virtual bool IsInternallyThreadSafe( ) const override
	{ 
		return true;
	}

	virtual const TCHAR* GetDescriptiveName( ) override
	{
		return TEXT("TBB");
	}

protected:

	void OutOfMemory( uint64 Size, uint32 Alignment )
	{
		// this is expected not to return
		FPlatformMemory::OnOutOfMemory(Size, Alignment);
	}
};

#endif // TBBMALLOC_ENABLED