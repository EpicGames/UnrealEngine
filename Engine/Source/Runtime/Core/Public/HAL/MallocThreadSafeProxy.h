// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/MemoryBase.h"
#include "Misc/ScopeLock.h"

/**
 * FMalloc proxy that synchronizes access, making the used malloc thread safe.
 */
class FMallocThreadSafeProxy : public FMalloc
{
private:
	/** Malloc we're based on, aka using under the hood							*/
	FMalloc*			UsedMalloc;
	/** Object used for synchronization via a scoped lock						*/
	FCriticalSection	SynchronizationObject;

public:
	/**
	 * Constructor for thread safe proxy malloc that takes a malloc to be used and a
	 * synchronization object used via FScopeLock as a parameter.
	 * 
	 * @param	InMalloc					FMalloc that is going to be used for actual allocations
	 */
	FMallocThreadSafeProxy( FMalloc* InMalloc)
	:	UsedMalloc( InMalloc )
	{}

	virtual void InitializeStatsMetadata() override
	{
		UsedMalloc->InitializeStatsMetadata();
	}

	/** 
	 * Malloc
	 */
	virtual void* Malloc( SIZE_T Size, uint32 Alignment ) override
	{
		FScopeLock ScopeLock( &SynchronizationObject );
		return UsedMalloc->Malloc( Size, Alignment );
	}

	/** 
	 * Realloc
	 */
	virtual void* Realloc( void* Ptr, SIZE_T NewSize, uint32 Alignment ) override
	{
		FScopeLock ScopeLock( &SynchronizationObject );
		return UsedMalloc->Realloc( Ptr, NewSize, Alignment );
	}

	/** 
	 * Free
	 */
	virtual void Free( void* Ptr ) override
	{
		if( Ptr )
		{
			FScopeLock ScopeLock( &SynchronizationObject );
			UsedMalloc->Free( Ptr );
		}
	}

	/** Writes allocator stats from the last update into the specified destination. */
	virtual void GetAllocatorStats( FGenericMemoryStats& out_Stats ) override
	{
		FScopeLock ScopeLock( &SynchronizationObject );
		UsedMalloc->GetAllocatorStats( out_Stats );
	}

	/** Dumps allocator stats to an output device. */
	virtual void DumpAllocatorStats( class FOutputDevice& Ar ) override
	{
		FScopeLock ScopeLock( &SynchronizationObject );
		UsedMalloc->DumpAllocatorStats( Ar );
	}

	/**
	 * Validates the allocator's heap
	 */
	virtual bool ValidateHeap() override
	{
		FScopeLock Lock( &SynchronizationObject );
		return( UsedMalloc->ValidateHeap() );
	}

#if UE_ALLOW_EXEC_COMMANDS
	virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) override
	{
		FScopeLock ScopeLock( &SynchronizationObject );
		return UsedMalloc->Exec( InWorld, Cmd, Ar);
	}
#endif // UE_ALLOW_EXEC_COMMANDS

	/**
	* If possible determine the size of the memory allocated at the given address
	*
	* @param Original - Pointer to memory we are checking the size of
	* @param SizeOut - If possible, this value is set to the size of the passed in pointer
	* @return true if succeeded
	*/
	virtual bool GetAllocationSize(void *Original, SIZE_T &SizeOut) override
	{
		FScopeLock ScopeLock( &SynchronizationObject );
		return UsedMalloc->GetAllocationSize(Original,SizeOut);
	}

	virtual const TCHAR* GetDescriptiveName() override
	{ 
		FScopeLock ScopeLock( &SynchronizationObject );
		check(UsedMalloc);
		return UsedMalloc->GetDescriptiveName(); 
	}

	virtual void Trim(bool bTrimThreadCaches) override
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		check(UsedMalloc);
		UsedMalloc->Trim(bTrimThreadCaches);
	}

	virtual bool IsInternallyThreadSafe() const override
	{ 
		return true; 
	}

	virtual void OnMallocInitialized() override
	{
		UsedMalloc->OnMallocInitialized();
	}

	virtual void OnPreFork() override
	{
		UsedMalloc->OnPreFork();
	}

	virtual void OnPostFork() override
	{
		UsedMalloc->OnPostFork();
	}
};
