// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformTime.h"

#ifndef UE_TIME_VIRTUALMALLOC
#define UE_TIME_VIRTUALMALLOC 0
#endif

struct FScopedVirtualMallocTimer
{
	enum IndexType : int32 
	{
		Reserve,
		Commit,
		Combined,
		DeCommit,
		Free,

		Max
	};

	enum PlatormIndexType : int32
	{
		OrdinaryCPU, // General memory
		GPU_WriteCombine, // XALLOC_MEMTYPE_GRAPHICS_COMMAND_BUFFER_WRITECOMBINE and XALLOC_MEMTYPE_GRAPHICS_WRITECOMBINE
		GPU_Cacheable, // XALLOC_MEMTYPE_GRAPHICS_CACHEABLE
		GPU_WriteCombineRenderTarget, // Similar to GPU_WriteCombine, but with 4MB pages and up to 128K alignment, no small block allocator
		PlatormIndexTypeMax
	};

#if UE_TIME_VIRTUALMALLOC
	static CORE_API uint64 GTotalCycles[IndexType::Max][PlatormIndexType::PlatormIndexTypeMax];
	static CORE_API uint64 GTotalCounts[IndexType::Max][PlatormIndexType::PlatormIndexTypeMax];

	int32 Index;
	int32 PlatformTypeIndex;
	uint64 Cycles;

	inline FScopedVirtualMallocTimer(int32 InIndex = 0, int32 InPlatformTypeIndex = 0)
		: Index(InIndex)
		, PlatformTypeIndex(InPlatformTypeIndex)
		, Cycles(FPlatformTime::Cycles64())
	{
		FPlatformAtomics::InterlockedIncrement((volatile int64*)&GTotalCounts[InIndex][InPlatformTypeIndex]);
	}
	inline ~FScopedVirtualMallocTimer()
	{
		uint64 Add = uint64(FPlatformTime::Cycles64() - Cycles);
		FPlatformAtomics::InterlockedAdd((volatile int64*)&GTotalCycles[Index][PlatformTypeIndex], Add);
	}

	static CORE_API void UpdateStats();
#else	//UE_TIME_VIRTUALMALLOC
	UE_FORCEINLINE_HINT FScopedVirtualMallocTimer(int32 InIndex = 0, int32 InPlatformTypeIndex = 0)
	{
	}
	UE_FORCEINLINE_HINT ~FScopedVirtualMallocTimer() = default;
	static UE_FORCEINLINE_HINT void UpdateStats()
	{
	}
#endif	//UE_TIME_VIRTUALMALLOC
};
