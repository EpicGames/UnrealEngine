// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if defined(PLATFORM_USES_FIXED_GMalloc_CLASS) && PLATFORM_USES_FIXED_GMalloc_CLASS
#	if USE_MALLOC_BINNED2
#		include "HAL/MallocBinned2.h" // HEADER_UNIT_IGNORE - Causes circular includes
#		define FMEMORY_INLINE_GMalloc (FMallocBinned2::MallocBinned2)
#	elif USE_MALLOC_BINNED3
#		include "HAL/MallocBinned3.h" // HEADER_UNIT_IGNORE - Causes circular includes
#		define FMEMORY_INLINE_GMalloc (FMallocBinned3::MallocBinned3)
#	endif
#endif

#if !defined(FMEMORY_INLINE_GMalloc)
#	include "CoreGlobals.h"
#	define FMEMORY_INLINE_GMalloc UE::Private::GMalloc
#endif

#include "HAL/FMemory.inl"