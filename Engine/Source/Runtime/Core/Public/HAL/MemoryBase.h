// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PlatformCrt.h"
#include "Misc/Exec.h"
#include "Misc/OutputDevice.h"
#include "Templates/Atomic.h"

class UWorld;
template <typename T> class TAtomic;

#ifndef UPDATE_MALLOC_STATS
	#define UPDATE_MALLOC_STATS 1
#endif

enum
{
	// Default allocator alignment. If the default is specified, the allocator applies to engine rules.
	// Blocks >= 16 bytes will be 16-byte-aligned, Blocks < 16 will be 8-byte aligned. If the allocator does
	// not support allocation alignment, the alignment will be ignored.
	DEFAULT_ALIGNMENT = 0,

	// Minimum allocator alignment
	MIN_ALIGNMENT = 8,
};

	
/** 
 * The global memory allocator.
 * Most callers should use FMemory::Malloc instead of accessing GMalloc directly.
 * FMemory wraps GMalloc but also provides low-level memory tracking and AutoRTFM support.
 */
namespace UE::Private
{
	CORE_API extern TSAN_ATOMIC(class FMalloc*) GMalloc;
}

/** 
 * Un-namespaced GMalloc remains exposed for backwards compatibility.
 * Most callers should use FMemory::Malloc instead of accessing GMalloc directly.
 * FMemory wraps GMalloc but also provides low-level memory tracking and AutoRTFM support.
 */
UE_DEPRECATED(5.6, "Consider using FMemory::Malloc instead of accessing GMalloc directly.")
CORE_API extern class FMalloc* const& GMalloc;

/** Memory allocator pointer location when PLATFORM_USES_FIXED_GMalloc_CLASS is true. */
CORE_API extern class FMalloc** GFixedMallocLocationPtr;

/** Holds generic memory stats, internally implemented as a map. */
struct FGenericMemoryStats;


/**
 * Inherit from FUseSystemMallocForNew if you want your objects to be placed in memory
 * alloced by the system malloc routines, bypassing GMalloc. This is e.g. used by FMalloc
 * itself.
 */
class FUseSystemMallocForNew
{
public:
	/**
	 * Overloaded new operator using the system allocator.
	 *
	 * @param	Size	Amount of memory to allocate (in bytes)
	 * @return			A pointer to a block of memory with size Size or nullptr
	 */
	CORE_API void* operator new(size_t Size);

	/**
	 * Overloaded delete operator using the system allocator
	 *
	 * @param	Ptr		Pointer to delete
	 */
	CORE_API void operator delete(void* Ptr);

	/**
	 * Overloaded array new operator using the system allocator.
	 *
	 * @param	Size	Amount of memory to allocate (in bytes)
	 * @return			A pointer to a block of memory with size Size or nullptr
	 */
	void* operator new[](size_t Size);

	/**
	 * Overloaded array delete operator using the system allocator
	 *
	 * @param	Ptr		Pointer to delete
	 */
	void operator delete[](void* Ptr);
};

/** The global memory allocator's interface. */
class FMalloc  : 
	public FUseSystemMallocForNew,
	public FExec
{
public:
	/**
	 * Malloc
	 */
	virtual void* Malloc( SIZE_T Count, uint32 Alignment=DEFAULT_ALIGNMENT ) = 0;

	/**
	 * TryMalloc - like Malloc(), but may return a nullptr result if the allocation
	 *             request cannot be satisfied.
	 */
	CORE_API virtual void* TryMalloc( SIZE_T Count, uint32 Alignment=DEFAULT_ALIGNMENT );

	/** 
	 * Realloc
	 */
	virtual void* Realloc( void* Original, SIZE_T Count, uint32 Alignment=DEFAULT_ALIGNMENT ) = 0;

	/** 
	 * TryRealloc - like Realloc(), but may return a nullptr if the allocation
	 *              request cannot be satisfied. Note that in this case the memory
	 *              pointed to by Original will still be valid
	 */
	CORE_API virtual void* TryRealloc(void* Original, SIZE_T Count, uint32 Alignment=DEFAULT_ALIGNMENT);

	/** 
	 * Free
	 */
	virtual void Free( void* Original ) = 0;

	/**
	 * Malloc zeroed memory
	 */
	CORE_API virtual void* MallocZeroed(SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT);

	/**
	 * TryMalloc - like MallocZeroed(), but may return a nullptr result if the allocation
	 *             request cannot be satisfied.
	 */
	CORE_API virtual void* TryMallocZeroed(SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT);
		
	/** 
	* For some allocators this will return the actual size that should be requested to eliminate
	* internal fragmentation. The return value will always be >= Count. This can be used to grow
	* and shrink containers to optimal sizes.
	* This call is always fast and threadsafe with no locking.
	*/
	virtual SIZE_T QuantizeSize(SIZE_T Count, uint32 Alignment)
	{
		return Count; // Default implementation has no way of determining this
	}

	/**
	* If possible determine the size of the memory allocated at the given address
	*
	* @param Original - Pointer to memory we are checking the size of
	* @param SizeOut - If possible, this value is set to the size of the passed in pointer
	* @return true if succeeded
	*/
	virtual bool GetAllocationSize(void *Original, SIZE_T &SizeOut)
	{
		return false; // Default implementation has no way of determining this
	}

	/**
	* Releases as much memory as possible. Must be called from the main thread.
	*/
	virtual void Trim(bool bTrimThreadCaches)
	{
	}

	/**
	* Set up TLS caches on the current thread. These are the threads that we can trim.
	*/
	virtual void SetupTLSCachesOnCurrentThread()
	{
	}

	/**
	* Mark TLS caches for the current thread as used. Thread has woken up to do some processing and needs its TLS caches back.
	*/
	virtual void MarkTLSCachesAsUsedOnCurrentThread()
	{
	}

	/**
	* Mark TLS caches for current thread as unused. Typically before going to sleep. These are the threads that we can trim without waking them up.
	*/
	virtual void MarkTLSCachesAsUnusedOnCurrentThread()
	{
	}

	/**
	* Clears the TLS caches on the current thread and disables any future caching.
	*/
	virtual void ClearAndDisableTLSCachesOnCurrentThread()
	{
	}

	/**
	*	Initializes stats metadata. We need to do this as soon as possible, but cannot be done in the constructor
	*	due to the FName::StaticInit
	*/
	CORE_API virtual void InitializeStatsMetadata();

#if UE_ALLOW_EXEC_COMMANDS
	/**
	 * Handles any commands passed in on the command line
	 */
	virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) override
	{ 
		return false; 
	}
#endif // UE_ALLOW_EXEC_COMMANDS

	/** Called once per frame, gathers and sets all memory allocator statistics into the corresponding stats. MUST BE THREAD SAFE. */
	CORE_API virtual void UpdateStats();

	/** Writes allocator stats from the last update into the specified destination. */
	CORE_API virtual void GetAllocatorStats( FGenericMemoryStats& out_Stats );

	/** Dumps current allocator stats to the log. */
	virtual void DumpAllocatorStats( class FOutputDevice& Ar )
	{
		Ar.Logf( TEXT("Allocator Stats for %s: (not implemented)" ), GetDescriptiveName() );
	}

	/**
	 * Returns if the allocator is guaranteed to be thread-safe and therefore
	 * doesn't need a unnecessary thread-safety wrapper around it.
	 */
	virtual bool IsInternallyThreadSafe() const 
	{ 
		return false; 
	}

	/**
	 * Validates the allocator's heap
	 */
	virtual bool ValidateHeap()
	{
		return( true );
	}

	/**
	 * Gets descriptive name for logging purposes.
	 *
	 * @return pointer to human-readable malloc name
	 */
	virtual const TCHAR* GetDescriptiveName()
	{
		return TEXT("Unspecified allocator");
	}

	/**
	 * Notifies the malloc implementation that initialization of all allocators in GMalloc is complete, so it's safe to initialize any extra features that require "regular" allocations
	 */
	virtual void OnMallocInitialized() {}

	/**
	 * Notifies the malloc implementation that the process is about to fork. May be used to trim caches etc.
	 */
	virtual void OnPreFork() {}

	/**
	 * Notifies the malloc implementation that the process has forked so we can try and avoid dirtying pre-fork pages.
	 */
	virtual void OnPostFork() {}

	/**
	 * Returns the amount of free memory cached by the allocator that can be returned to the system in case of a memory shortage
	 */
	virtual uint64 GetImmediatelyFreeableCachedMemorySize() const
	{
		return 0;
	}

	/**
	 * Returns the amount of total free memory cached by the allocator.
	 * This includes memory that can be returned to the system in case of a memory shortage, see GetImmediatelyFreeableCachedMemorySize()
	 * as well as any memory that can't be returned back to the kernel, but can be used to satisfy some of the allocation requirements.
	 */
	virtual uint64 GetTotalFreeCachedMemorySize() const
	{
		return 0;
	}

protected:
	friend struct FCurrentFrameCalls;

#if !UE_BUILD_SHIPPING
public:
	/** Limits the maximum single allocation, to this many bytes, for debugging */
	static CORE_API TAtomic<uint64> MaxSingleAlloc;
#endif
};
