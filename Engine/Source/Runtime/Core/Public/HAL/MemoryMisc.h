// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "Misc/StringBuilder.h"

#ifndef UE_MEMORY_STAT_DESCRIPTION_LENGTH_DEFAULT
#define UE_MEMORY_STAT_DESCRIPTION_LENGTH 64
#endif

#ifndef UE_MEMORY_STAT_PREALLOCATION_COUNT
#define UE_MEMORY_STAT_PREALLOCATION_COUNT 32
#endif

/** Holds generic memory stats, internally implemented as a map. */
struct FGenericMemoryStats
{
private:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static constexpr int32 EntryCount = UE_MEMORY_STAT_PREALLOCATION_COUNT;
	static constexpr int32 StringLength = UE_MEMORY_STAT_DESCRIPTION_LENGTH;
	using FInternalMap = TMap<FStringView, SIZE_T, TInlineSetAllocator<EntryCount>>;

public:
	void Add(const FStringView& InDescription, SIZE_T InValue)
	{
		const FStringView DescriptionView = Descriptions.Emplace_GetRef(InPlace, InDescription).ToView();
		Data.FInternalMap::Add(DescriptionView, InValue);
	}

	SIZE_T* Find(const FStringView& InDescription)
	{
		return Data.FInternalMap::Find(InDescription);
	}

	const SIZE_T* Find(const FStringView& InDescription) const
	{
		return const_cast<FGenericMemoryStats*>(this)->Find(InDescription);
	}

	SIZE_T FindRef(const FStringView& InDescription) const
	{
		return Data.FInternalMap::FindRef(InDescription);
	}

	// Expose iterators only for const ranged-for iteration
	FInternalMap::TRangedForConstIterator begin() const { return Data.begin(); }
	FInternalMap::TRangedForConstIterator end() const { return Data.end(); }

private:
	TArray<TStringBuilder<StringLength>, TInlineAllocator<EntryCount>> Descriptions;

	/** Wrapper on a TMap<TStringView> to allow passing both ANSI and TCHAR strings, for deprecation phase */
	class FGenericMemoryStatsMap : public FInternalMap
	{
	public:
		SIZE_T* Find(const FString& InDescription)
		{
			const FStringView DescriptionView(InDescription);
			return FInternalMap::Find(DescriptionView);
		}

		const SIZE_T* Find(const FString& InDescription) const
		{
			return const_cast<FGenericMemoryStatsMap*>(this)->Find(InDescription);
		}

		SIZE_T FindRef(const FString& InDescription) const
		{
			const FStringView DescriptionView(InDescription);
			return FInternalMap::FindRef(DescriptionView);
		}
	};
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:
	UE_DEPRECATED(5.5, "Using Data directly is deprecated and will be made private. Use methods on FGenericMemoryStats directly.")
	FGenericMemoryStatsMap Data;
};

#ifndef ENABLE_MEMORY_SCOPE_STATS
#define ENABLE_MEMORY_SCOPE_STATS 0
#endif

// This will grab the memory stats of VM and Physical before and at the end of scope
// reporting +/- difference in memory.
// WARNING This will also capture differences in Threads which have nothing to do with the scope
#if ENABLE_MEMORY_SCOPE_STATS
class FScopedMemoryStats
{
public:
	CORE_API explicit FScopedMemoryStats(const TCHAR* Name);

	CORE_API ~FScopedMemoryStats();

private:
	const TCHAR* Text;
	const FPlatformMemoryStats StartStats;
};
#else
class FScopedMemoryStats
{
public:
	explicit FScopedMemoryStats(const TCHAR* Name) {}
};
#endif

/**
 * The FSharedMemoryTracker is used to track how much the shared and unique memory pools changed size in-between each call
 * WARNING: Getting the shared & unique memory pool size is extremely costly (easily takes up to 60ms) so be careful
 * not to use the tracker while hosting a game.
 */
#ifndef ENABLE_SHARED_MEMORY_TRACKER
#define ENABLE_SHARED_MEMORY_TRACKER 0
#endif

#if ENABLE_SHARED_MEMORY_TRACKER && PLATFORM_UNIX

class FSharedMemoryTracker
{
public:

	/** Print the difference in memory pool sizes since the last call to this function exclusively */
	static CORE_API void PrintMemoryDiff(const TCHAR* Context);


	/** Store the memory pool size at construction and log the difference in memory that occurred during the lifetime of the tracker */
	CORE_API explicit FSharedMemoryTracker(const FString& InContext);
	CORE_API ~FSharedMemoryTracker();

private:
	const FString PrintContext;
	const FExtendedPlatformMemoryStats StartStats;
};
#else
class FSharedMemoryTracker
{
public:

	static void PrintMemoryDiff(const TCHAR* /*Context*/) {}

	explicit FSharedMemoryTracker(const FString& /*InContext*/) {}
};
#endif // ENABLE_SHARED_MEMORY_TRACKER  && PLATFORM_UNIX
