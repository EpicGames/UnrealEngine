// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/AlignmentTemplates.h"


#ifndef MIP_FIELD_ALLOCATOR_DEBUG
#	define MIP_FIELD_ALLOCATOR_DEBUG 0
#endif

#if MIP_FIELD_ALLOCATOR_DEBUG
#	define MipCheck(x) check(x)
#else
#	define MipCheck(...)
#endif

//
// TMipField is a binary tree bit field container, used by the page allocators.
//
// Individual bits in the tree are addressable via a (mip, slot, bit) scheme.
// Each level of the tree (aka "mip") contains multiple "slots". Each "slot" contains 'n' bits.
//
// The following diagram shows a TMipField which represents 32 pages, with 1 bit per slot.
//
//									                              Mip Level
//                                1					                  0
//                1-------------------------------1			          1
//        1---------------1               1---------------1		      2
//    1-------1       1-------1       1-------1       *-------1		  3
//  1---1   1---1   1---1   1---1   1---1   1---1   1---1   1---1	  4
// 1-1 1-1 1-1 1-1 1-1 1-1 1-1 1-1 1-1 1-1 1-1 1-1 1-1 1-1 1-1 1-1    5
//
//                     1 1 1 1 1 1 1 1 1 1 2 2 2 2 2 2 2 2 2 2 3 3
// 0 1 2 3 4 5 6 7 8 9 0 1 2 3 4 5 6 7 8 9 0 1 2 3 4 5 6 7 8 9 0 1  Page Index
//
// For example, the highlighted bit (*) in the above tree has a (mip, slot, bit) address of (3, 6, 0).
// That particular bit represents a 4 page region, starting at the 24th page (i.e. pages 24, 25, 26, and 27).
//
// The TMipFieldAllocator class uses a TMipField to store 2 bits per page: "ANY" and "ALL".
// When these bits are set to '1', they indicate that "any" or "all" child slots have the same bit set.
// e.g. If the highlighted slot's "ALL" bit was '1', this indicates the entire 4-page region (24 to 27) is allocated.
//      Likewise, if the "ANY" bit is set, this indicates at least 1 page in this region is allocated.
//
// The storage for the binary tree structure is provided by the TAllocator template argument. This allows for TMipField
// instances to use pre-allocated space at compile time before the general purpose allocators are initialized.
//

template <typename TAllocator, uint64 InBitsPerSlot>
struct TMipField
{
	// Computes the total number of elements across all mips in the mipfield.
	static constexpr inline uint64 GetTotalNumElements(uint64 NumSlotsThisMip)
	{
		uint64 ParentMipNumElements = (NumSlotsThisMip > 1) ? GetTotalNumElements(NumSlotsThisMip / 2) : 0; // Recurse up to next mip (half size, hence divide by 2)
		uint64 ThisMipNumElements = (NumSlotsThisMip + (SlotsPerElement - 1)) / SlotsPerElement; // Round up

		return ThisMipNumElements + ParentMipNumElements;
	}

	// Underlying storage type.
	typedef uint8_t FMipFieldElement;

	// Total number of slots in the largest mip.
	uint64 const NumSlots;

	// total number of bits to allocate for each slot.
	static constexpr uint64 const BitsPerSlot = InBitsPerSlot;

	// Number of bits in a single element
	static constexpr uint64 const BitsPerElement = sizeof(FMipFieldElement) * 8;

	// The number of unique slots that can be packed into a single element.
	static constexpr uint64 const SlotsPerElement = BitsPerElement / InBitsPerSlot;

	// The total number of levels in the mip chain.
	uint64 const NumLevels;

	// The total number of elements, which provide the storage for the entire mip chain.
	uint64 const NumElements;

	FMipFieldElement* Elements;
	uint64* MipLevelOffsets;

	inline void AddressSingleMipBit(uint64 Mip, uint64 Slot, uint64 Bit, uint64& OutElementIndex, uint64& OutBitIndex) const
	{
		MipCheck(Mip < NumLevels);

		uint64 SlotsThisMip = uint64(1) << Mip;
		MipCheck(Slot < SlotsThisMip);

		OutElementIndex = MipLevelOffsets[Mip] + (Slot / SlotsPerElement);
		OutBitIndex = ((Slot % SlotsPerElement) * BitsPerSlot) + Bit;
	}

	inline bool GetMipBit(uint64 Mip, uint64 Slot, uint64 Bit) const
	{
		uint64 ElementIndex, BitIndex;
		AddressSingleMipBit(Mip, Slot, Bit, ElementIndex, BitIndex);

		return (Elements[ElementIndex] & (uint64(1) << BitIndex)) != 0;
	}

	// Also returns the old value of the bit
	inline bool SetMipBit(uint64 Mip, uint64 Slot, uint64 Bit, bool Value)
	{
		uint64 ElementIndex, BitIndex;
		AddressSingleMipBit(Mip, Slot, Bit, ElementIndex, BitIndex);

		FMipFieldElement OldElementValue = Elements[ElementIndex];
		uint64 Mask = uint64(1) << BitIndex;
		if (Value)
		{
			Elements[ElementIndex] = static_cast<FMipFieldElement>(OldElementValue | Mask);
		}
		else
		{
			Elements[ElementIndex] &= (~Mask);
		}

		return !!(OldElementValue & Mask);
	}

	TMipField(uint64 InNumSlots)
		: NumSlots(InNumSlots)
		, NumLevels(FMath::FloorLog2_64(NumSlots) + 1)
		, NumElements(GetTotalNumElements(NumSlots))
		, Elements(nullptr)
		, MipLevelOffsets(nullptr)
	{
		checkf(FMath::IsPowerOfTwo(NumSlots), TEXT("Number of slots in a mipfield must be a power of two."));
		checkf(FMath::IsPowerOfTwo(InBitsPerSlot), TEXT("Bits per slot must be a power of two."));
		checkf((sizeof(FMipFieldElement) * 8) >= InBitsPerSlot, TEXT("Mipfield element must be greater than or equal to the number of bits per slot."));

		Elements = reinterpret_cast<FMipFieldElement*>(TAllocator::Allocate(NumElements * sizeof(FMipFieldElement)));
		MipLevelOffsets = reinterpret_cast<uint64*>(TAllocator::Allocate(NumLevels * sizeof(uint64)));

		FMemory::Memzero(Elements, NumElements * sizeof(FMipFieldElement));

		MipLevelOffsets[0] = 0;
		for (uint64 Index = 0; Index < NumLevels - 1; ++Index)
		{
			MipLevelOffsets[Index + 1] = GetTotalNumElements(uint64(1) << Index);
		}
	}

	~TMipField()
	{
		TAllocator::Free(Elements);
		TAllocator::Free(MipLevelOffsets);
	}
};


template <typename TAllocator>
struct TMipFieldPageAllocator
{
	uint64 const TotalNumPages;

	TMipFieldPageAllocator(uint64 InTotalNumPages)
		: TotalNumPages(InTotalNumPages)
		, MipField(TotalNumPages)
	{
	}

public:
	enum
	{
		ANY = 0,
		ALL = 1
	};

	// 2 bits per page, "any" and "all"
	typedef TMipField<TAllocator, 2> TMipFieldType;
	TMipFieldType MipField;

	// Returns the absolute page index of the first page covered by the specified slot in the mip.
	inline uint64 MipSlot_To_PageIndex(uint64 Mip, uint64 Slot) const
	{
		return Slot << ((MipField.NumLevels - 1) - Mip);
	}

	// Finds the slot in the highest mip which covers the page region
	inline void PageIndexSize_To_MipSlot(uint64 PageIndex, uint64 Size, uint64& OutMip, uint64& OutSlot) const
	{
		OutMip = MipField.NumLevels - (FMath::FloorLog2_64(Size) + 1);
		OutSlot = PageIndex / Size;
	}

	static inline uint64 GetSiblingSlot(uint64 Slot)
	{
		// Flip the bottom bit to find the sibling slot
		return ((~Slot) & 0x01) | (Slot & ((~uint64(0)) << 1));
	}

	// Returns the number of pages covered by a single slot in the given mip.
	inline uint64 GetPagesInSingleMipSlot(uint64 Mip) const
	{
		return TotalNumPages >> Mip;
	}

#if MIP_FIELD_ALLOCATOR_DEBUG
	void CheckTreeRecursive(uint64 Mip, uint64 LeftSlot, bool ParentAny, bool ParentAll) const
	{
		bool LeftAny = MipField.GetMipBit(Mip, LeftSlot, ANY);
		bool RightAny = MipField.GetMipBit(Mip, LeftSlot + 1, ANY);

		bool LeftAll = MipField.GetMipBit(Mip, LeftSlot, ALL);
		bool RightAll = MipField.GetMipBit(Mip, LeftSlot + 1, ALL);

		if (LeftAny || RightAny) { MipCheck(ParentAny); }
		if (LeftAll && RightAll) { MipCheck(ParentAll); }

		if (LeftAll) { MipCheck(LeftAny); }
		if (RightAll) { MipCheck(RightAny); }

		if (Mip < MipField.NumLevels - 1)
		{
			CheckTreeRecursive(Mip + 1, LeftSlot << 1, LeftAny, LeftAll);
			CheckTreeRecursive(Mip + 1, (LeftSlot + 1) << 1, RightAny, RightAll);
		}
	}

	void CheckTree() const
	{
		// Root node is special case
		bool RootAny = MipField.GetMipBit(0, 0, ANY);
		bool RootAll = MipField.GetMipBit(0, 0, ALL);
		if (RootAll) { MipCheck(RootAny); }

		CheckTreeRecursive(1, 0, RootAny, RootAll);
	}

	void CheckPages(uint64 PageIndex, uint64 NumPages, bool IsAllocated) const
	{
		// Verify region is already set/free
		for (uint64 Index = 0; Index < NumPages; ++Index)
		{
			bool AnyBit = MipField.GetMipBit(MipField.NumLevels - 1, Index + PageIndex, ANY);
			bool AllBit = MipField.GetMipBit(MipField.NumLevels - 1, Index + PageIndex, ALL);

			MipCheck(AnyBit == AllBit);
			MipCheck(AnyBit == IsAllocated);
		}
	}
#endif

	inline void AssignSubtree(uint64 MipLevel, uint64 Slot, bool Value)
	{
#if MIP_FIELD_ALLOCATOR_DEBUG
		// Check this bit *is not* already set to [Value]
		//MipCheck(MipField.GetMipBit(MipLevel, Slot, ANY) != Value);
		if (MipField.GetMipBit(MipLevel, Slot, ANY) == Value)
		{
			uint64 ElementIndex, BitIndex, BaseElement, BaseBit;
			uint64 PageIndex = MipSlot_To_PageIndex(MipLevel, Slot);
			MipField.AddressSingleMipBit(MipLevel, Slot, ANY, ElementIndex, BitIndex);
			MipField.AddressSingleMipBit(MipField.NumLevels - 1, PageIndex, ANY, BaseElement, BaseBit);
			printf("AssignSubtree FAILURE: PageIndex %llu\r\n"
				"                       Mip %llu, Slot %llu\r\n"
				"                       Element %llu, Bit %llu\r\n"
				"                  Base Element %llu, Bit %llu\r\n",
				PageIndex, MipLevel, Slot, ElementIndex, BitIndex, BaseElement, BaseBit);
			PLATFORM_BREAK();
		}

		//MipCheck(MipField.GetMipBit(MipLevel, Slot, ALL) != Value);
		if (MipField.GetMipBit(MipLevel, Slot, ALL) == Value)
		{
			uint64 ElementIndex, BitIndex, BaseElement, BaseBit;
			uint64 PageIndex = MipSlot_To_PageIndex(MipLevel, Slot);
			MipField.AddressSingleMipBit(MipLevel, Slot, ALL, ElementIndex, BitIndex);
			MipField.AddressSingleMipBit(MipField.NumLevels - 1, PageIndex, ALL, BaseElement, BaseBit);
			printf("AssignSubtree FAILURE: PageIndex %llu\r\n"
				"                       Mip %llu, Slot %llu\r\n"
				"                       Element %llu, Bit %llu\r\n"
				"                  Base Element %llu, Bit %llu\r\n",
				PageIndex, MipLevel, Slot, ElementIndex, BitIndex, BaseElement, BaseBit);
			PLATFORM_BREAK();
		}
#endif

		if (MipLevel < MipField.NumLevels - 1)
		{
			// Recurse through tree to set bits
			AssignSubtree(MipLevel + 1, Slot << 1, Value);
			AssignSubtree(MipLevel + 1, (Slot << 1) + 1, Value);
		}

		MipField.SetMipBit(MipLevel, Slot, ANY, Value);
		MipField.SetMipBit(MipLevel, Slot, ALL, Value);
	}

	inline void FixupParents(uint64 StartMip, uint64 StartSlot)
	{
		uint64 CurrentMip = StartMip;
		uint64 CurrentSlot = StartSlot;

		// TODO: Early out of this loop when we hit a part of
		// the tree which is already at the expected values.
		while (CurrentMip >= 1)
		{
			uint64 ParentMip = CurrentMip - 1;
			uint64 ParentSlot = CurrentSlot >> 1;
			uint64 SiblingSlot = GetSiblingSlot(CurrentSlot);

			bool SiblingAny = MipField.GetMipBit(CurrentMip, SiblingSlot, ANY);
			bool SiblingAll = MipField.GetMipBit(CurrentMip, SiblingSlot, ALL);

			bool ThisAny = MipField.GetMipBit(CurrentMip, CurrentSlot, ANY);
			bool ThisAll = MipField.GetMipBit(CurrentMip, CurrentSlot, ALL);

			bool NewParentAny = ThisAny || SiblingAny;
			bool NewParentAll = ThisAll && SiblingAll;

			MipField.SetMipBit(ParentMip, ParentSlot, ANY, NewParentAny); // Update parent ANY bit.
			MipField.SetMipBit(ParentMip, ParentSlot, ALL, NewParentAll); // Update parent ALL bit.

			CurrentMip = ParentMip;
			CurrentSlot = ParentSlot;
		}
	}

	bool FindFreeRegionRecursive_PowerOfTwo(uint64 NumPages, uint64 MipLevel, uint64 LeftSlot, uint64& OutMipLevel, uint64& OutSlot) const
	{
		uint64 PagesInSlot = TotalNumPages >> MipLevel;
		uint64 RightSlot = LeftSlot + 1;

		if (PagesInSlot > NumPages)
		{
			// We've not reached the bottom of the tree yet.
			// Try searching left, only if ALL bit is not set for that subtree
			if (!MipField.GetMipBit(MipLevel, LeftSlot, ALL) && FindFreeRegionRecursive_PowerOfTwo(NumPages, MipLevel + 1, LeftSlot << 1, OutMipLevel, OutSlot))
			{
				return true;
			}
			// If that failed, try the right subtree.
			else if (!MipField.GetMipBit(MipLevel, RightSlot, ALL) && FindFreeRegionRecursive_PowerOfTwo(NumPages, MipLevel + 1, RightSlot << 1, OutMipLevel, OutSlot))
			{
				return true;
			}
			else
			{
				// No space in either child
				return false;
			}
		}
		else
		{
			if (!MipField.GetMipBit(MipLevel, LeftSlot, ANY))
			{
				MipCheck(!MipField.GetMipBit(MipLevel, LeftSlot, ALL));

				// Found empty region in entire left tree
				OutMipLevel = MipLevel;
				OutSlot = LeftSlot;
				return true;
			}
			else if (!MipField.GetMipBit(MipLevel, RightSlot, ANY))
			{
				MipCheck(!MipField.GetMipBit(MipLevel, RightSlot, ALL));

				// Found empty region in entire right tree
				OutMipLevel = MipLevel;
				OutSlot = RightSlot;
				return true;
			}
			else
			{
				// No space in either child
				return false;
			}
		}
	}

	bool FindFreeRegion_PowerOfTwo(uint64 NumPages, uint64& OutMipLevel, uint64& OutSlot) const
	{
		MipCheck(NumPages != 0 && FMath::IsPowerOfTwo(NumPages));
		MipCheck(NumPages <= TotalNumPages);

		// Root node is a special case (it has no sibling)
		if (NumPages == TotalNumPages)
		{
			OutMipLevel = 0;
			OutSlot = 0;
			return MipField.GetMipBit(0, 0, ANY) == false;
		}

		return FindFreeRegionRecursive_PowerOfTwo(NumPages, 1, 0, OutMipLevel, OutSlot);
	}

	// Marks the specified region as either allocated or free. The region must not already be in the specified state.
	// NumPages must be a power of two, and PageIndex should be aligned so that it indexes the first page of a power-of-two region.
	// i.e. PageIndex is a leftmost child of a subtree.
	void SetRegion_PowerOfTwo(uint64 PageIndex, uint64 NumPages, bool IsAllocated)
	{
		MipCheck(NumPages > 0);
		MipCheck(NumPages <= TotalNumPages);
		MipCheck((NumPages + PageIndex) <= TotalNumPages);
		MipCheck(FMath::IsPowerOfTwo(NumPages));

		// Ensure the offset to the first page is aligned with the power-of-two region
		MipCheck(((NumPages - 1) & PageIndex) == 0);

		uint64 Mip, Slot;
		PageIndexSize_To_MipSlot(PageIndex, NumPages, Mip, Slot);

#if MIP_FIELD_ALLOCATOR_DEBUG
		CheckTree();
#endif

		AssignSubtree(Mip, Slot, IsAllocated);
		FixupParents(Mip, Slot);

#if MIP_FIELD_ALLOCATOR_DEBUG
		CheckTree();
#endif
	}

	inline void GetRegionBounds_ExpandRegion(uint64 PageIndex, bool IsAllocated, uint64& OutLowerBound, uint64& OutUpperBound) const
	{
		const uint64 BaseMip = MipField.NumLevels - 1;
		const uint64 TestBit = IsAllocated ? ALL : ANY;

		uint64 CurrentMip = BaseMip;
		uint64 CurrentSlot = PageIndex;

		while (CurrentMip >= 1 && MipField.GetMipBit(CurrentMip, GetSiblingSlot(CurrentSlot), TestBit) == IsAllocated)
		{
			CurrentMip--;
			CurrentSlot >>= 1;
		}

		// Get the bounds of this subtree
		OutLowerBound = MipSlot_To_PageIndex(CurrentMip, CurrentSlot);
		OutUpperBound = OutLowerBound + GetPagesInSingleMipSlot(CurrentMip);
	}

public:
	// Returns true if the allocator is completely full.
	inline bool IsFull() const { return MipField.GetMipBit(0, 0, ALL) == true; }

	// Returns true if the allocator is completely empty.
	inline bool IsEmpty() const { return MipField.GetMipBit(0, 0, ANY) == false; }

	// Returns true if the given arbitrary region is fully allocated.
	bool IsRegionFull(uint64 PageIndex, uint64 NumPages) const
	{
		while (NumPages > 0)
		{
			uint64 MaxPagesThisIteration = uint64(1) << FMath::Min(FMath::CountTrailingZeros64(PageIndex), MipField.NumLevels - 1);

			// Get the next power-of-two sized region to assign
			uint64 PagesToCheck = FMath::RoundDownToPowerOfTwo64(NumPages);
			PagesToCheck = FMath::Min(PagesToCheck, MaxPagesThisIteration);

			uint64 Mip, Slot;
			PageIndexSize_To_MipSlot(PageIndex, PagesToCheck, Mip, Slot);

			if (MipField.GetMipBit(Mip, Slot, ALL) == false)
			{
				return false;
			}

			PageIndex += PagesToCheck;
			NumPages -= PagesToCheck;
		}

		return true;
	}

	// For the given page index, computes the upper and lower bounds of the region
	// the specified page exists in, and returns if this region is free or allocated.
	bool GetRegionBounds(uint64 PageIndex, uint64& OutFirstPageIndex, uint64& OutNumPages) const
	{
		// Read the allocation bit for the current page
		const uint64 BaseMip = MipField.NumLevels - 1;
		const bool IsAllocated = MipField.GetMipBit(BaseMip, PageIndex, ANY);

		// Expand this region to the initial upper/lower bounds.
		uint64 LowerBound, UpperBound;
		GetRegionBounds_ExpandRegion(PageIndex, IsAllocated, LowerBound, UpperBound);

		// Whilst there is more room in the tree to the left, and the adjacent local region also has the same allocation type.
		while (LowerBound > 0 && MipField.GetMipBit(BaseMip, LowerBound - 1, ANY) == IsAllocated)
		{
			uint64 LocalLowerBound, LocalUpperBound;
			GetRegionBounds_ExpandRegion(LowerBound - 1, IsAllocated, LocalLowerBound, LocalUpperBound);

			MipCheck(LocalUpperBound == LowerBound);
			MipCheck(LocalLowerBound <= LowerBound);
			LowerBound = LocalLowerBound;
		}

		// Whilst there is more room in the tree to the right, and the adjacent local region also has the same allocation type.
		while (UpperBound < TotalNumPages && MipField.GetMipBit(BaseMip, UpperBound, ANY) == IsAllocated)
		{
			uint64 LocalLowerBound, LocalUpperBound;
			GetRegionBounds_ExpandRegion(UpperBound, IsAllocated, LocalLowerBound, LocalUpperBound);

			MipCheck(LocalLowerBound == UpperBound);
			MipCheck(UpperBound <= LocalUpperBound);
			UpperBound = LocalUpperBound;
		}

		OutFirstPageIndex = LowerBound;
		OutNumPages = UpperBound - LowerBound;

#if MIP_FIELD_ALLOCATOR_DEBUG
		// Verify result...
		if (LowerBound > 0)
		{
			MipCheck(MipField.GetMipBit(BaseMip, LowerBound - 1, ANY) != IsAllocated);
		}
		if (UpperBound < TotalNumPages)
		{
			MipCheck(MipField.GetMipBit(BaseMip, UpperBound, ANY) != IsAllocated);
		}
		for (uint64 Index = LowerBound; Index < UpperBound; ++Index)
		{
			MipCheck(MipField.GetMipBit(BaseMip, Index, ANY) == IsAllocated);
		}

		CheckPages(OutFirstPageIndex, OutNumPages, IsAllocated);
#endif

		return IsAllocated;
	}

	// Marks the specified arbitrary region as either allocated or free. The region must not already be in the specified state.
	void SetRegion(uint64 PageIndex, uint64 NumPages, bool IsAllocated)
	{
#if MIP_FIELD_ALLOCATOR_DEBUG
		CheckPages(PageIndex, NumPages, !IsAllocated);
#endif

		while (NumPages > 0)
		{
			uint64 MaxPagesThisIteration = uint64(1) << FMath::Min(FMath::CountTrailingZeros64(PageIndex), MipField.NumLevels - 1);

			// Get the next power-of-two sized region to assign
			uint64 PagesToAssign = FMath::RoundDownToPowerOfTwo64(NumPages);
			PagesToAssign = FMath::Min(PagesToAssign, MaxPagesThisIteration);

			// Modify the tree
			SetRegion_PowerOfTwo(PageIndex, PagesToAssign, IsAllocated);

			PageIndex += PagesToAssign;
			NumPages -= PagesToAssign;
		}

#if MIP_FIELD_ALLOCATOR_DEBUG
		CheckPages(PageIndex, NumPages, IsAllocated);
#endif
	}

	// Finds and allocates a region large enough to hold [NumPages], returning the page index in the "out" argument and true on success, otherwise false.
	bool AllocatePages(uint64 NumPages, uint64 AlignmentInPages, uint64& OutPageIndex)
	{
		// The buddy allocator can only allocate space between power-of-two boundaries.
		// Find the smallest power-of-two sized region large enough to hold correctly aligned NumPages.
		AlignmentInPages = FMath::Max(NumPages, AlignmentInPages);
		uint64 RoundedNumPages = FMath::RoundUpToPowerOfTwo64(AlignmentInPages);

		uint64 RegionStartMip, RegionStartSlot;
		if (FindFreeRegion_PowerOfTwo(RoundedNumPages, RegionStartMip, RegionStartSlot))
		{
			OutPageIndex = MipSlot_To_PageIndex(RegionStartMip, RegionStartSlot);
			SetRegion(OutPageIndex, NumPages, true);
			return true;
		}
		else
		{
			// No power-of-two region exists. Failed to allocate.
			OutPageIndex = 0;
			return false;
		}
	}

	// Frees the region covered by [PageIndex, NumPages]. The region must already have been allocated.
	void FreePages(uint64 PageIndex, uint64 NumPages)
	{
		SetRegion(PageIndex, NumPages, false);
	}

	static constexpr uint64 CalcMemoryRequirements(uint64 NumSlots)
	{
		const uint64 NumLevels = FMath::FloorLog2_64(NumSlots) + 1;
		const uint64 NumElements = TMipFieldType::GetTotalNumElements(NumSlots);

		uint64 Size = NumElements * sizeof(typename TMipFieldType::FMipFieldElement);
		Size += NumLevels * sizeof(uint64);
		Size = Align(Size, 8);
		return Size;
	}
};

// Allocator type for TMipFields which uses general purpose malloc.
// Used for runtime instances of TMipField, in the GPU small block allocator.
struct FMipFieldRunTimeAllocator
{
	static inline void* Allocate(uint64 Size) { return FPlatformMemory::BaseAllocator()->Malloc(Size); }
	static inline void Free(void* Ptr) { FPlatformMemory::BaseAllocator()->Free(Ptr); }
};

/*
	An example of a compile time allocator

struct FMipFieldCompileTimeAllocator
{
	static inline void* Allocate(uint64 Size)
	{
		// Linear allocate from the pool
		Size = Align(Size, 8);
		if ((Offset + Size) > MaxPoolSize)
			abort();

		void* Original = Pool + Offset;
		Offset += Size;
		return Original;
	}

	static inline void Free(void*)
	{
		// No-op
	}

private:
	typedef TMipFieldPageAllocator<FMipFieldCompileTimeAllocator> TAllocator;
	static constexpr uint64 MaxPoolSize = TAllocator::CalcMemoryRequirements((16ull * 1024ull * 1024ull * 1024ull) / 4096);
	static uint8 Pool[MaxPoolSize];
	static uint64 Offset;
};

uint8 FMipFieldCompileTimeAllocator::Pool[MaxPoolSize] = {};
uint64 FMipFieldCompileTimeAllocator::Offset = 0;

*/