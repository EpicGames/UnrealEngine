// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_UNSUPPORTED - Unsupported platform

#include "Misc/AssertionMacros.h"
#include "HAL/Event.h"

/**
 * This is the PThreads version of FEvent.
 */
class FPThreadEvent
	: public FEvent
{
	// This is a little complicated, in an attempt to match Win32 Event semantics...
    typedef enum
    {
        TRIGGERED_NONE,
        TRIGGERED_ONE,
        TRIGGERED_ALL,
    } TriggerType;

	bool bInitialized;
	bool bIsManualReset;
	volatile TriggerType Triggered;
	volatile int32 WaitingThreads;
    pthread_mutex_t Mutex;
    pthread_cond_t Condition;

	inline void LockEventMutex()
	{
		const int rc = pthread_mutex_lock(&Mutex);
		check(rc == 0);
	}

	inline void UnlockEventMutex()
	{
		const int rc = pthread_mutex_unlock(&Mutex);
		check(rc == 0);
	}
	
	static inline void SubtractTimevals( const struct timeval *FromThis, struct timeval *SubThis, struct timeval *Difference )
	{
		if (FromThis->tv_usec < SubThis->tv_usec)
		{
			int nsec = (SubThis->tv_usec - FromThis->tv_usec) / 1000000 + 1;
			SubThis->tv_usec -= 1000000 * nsec;
			SubThis->tv_sec += nsec;
		}

		if (FromThis->tv_usec - SubThis->tv_usec > 1000000)
		{
			int nsec = (FromThis->tv_usec - SubThis->tv_usec) / 1000000;
			SubThis->tv_usec += 1000000 * nsec;
			SubThis->tv_sec -= nsec;
		}

		Difference->tv_sec = FromThis->tv_sec - SubThis->tv_sec;
		Difference->tv_usec = FromThis->tv_usec - SubThis->tv_usec;
	}

public:

	FPThreadEvent()
	{
		bInitialized = false;
		bIsManualReset = false;
		Triggered = TRIGGERED_NONE;
		WaitingThreads = 0;
	}

	virtual ~FPThreadEvent()
	{
		// Safely destructing an Event is VERY delicate, so it can handle badly-designed
		//  calling code that might still be waiting on the event.
		if (bInitialized)
		{
			// try to flush out any waiting threads...
			LockEventMutex();
			bIsManualReset = true;
			UnlockEventMutex();
			Trigger();  // any waiting threads should start to wake up now.

			LockEventMutex();
			bInitialized = false;  // further incoming calls to this object will now crash in check().
			while (WaitingThreads)  // spin while waiting threads wake up.
			{
				UnlockEventMutex();  // cycle through waiting threads...
				LockEventMutex();
			}
			// No threads are currently waiting on Condition and we hold the Mutex. Kill it.
			pthread_cond_destroy(&Condition);
			// Unlock and kill the mutex, since nothing else can grab it now.
			UnlockEventMutex();
			pthread_mutex_destroy(&Mutex);
		}
	}

	// FEvent interface

	virtual bool Create( bool _bIsManualReset = false ) override
	{
		check(!bInitialized);
		bool RetVal = false;
		Triggered = TRIGGERED_NONE;
		bIsManualReset = _bIsManualReset;

		if (pthread_mutex_init(&Mutex, nullptr) == 0)
		{
			if (pthread_cond_init(&Condition, nullptr) == 0)
			{
				bInitialized = true;
				RetVal = true;
			}
			else
			{
				pthread_mutex_destroy(&Mutex);
			}
		}
		return RetVal;
	}

	virtual bool IsManualReset() override
	{
		return bIsManualReset;
	}

	virtual void Trigger() override
	{
		TriggerForStats();

		check(bInitialized);

		LockEventMutex();

		if (bIsManualReset)
		{
			// Release all waiting threads at once.
			Triggered = TRIGGERED_ALL;
			int rc = pthread_cond_broadcast(&Condition);
			check(rc == 0);
		}
		else
		{
			// Release one or more waiting threads (first one to get the mutex
			//  will reset Triggered, rest will go back to waiting again).
			Triggered = TRIGGERED_ONE;
			int rc = pthread_cond_signal(&Condition);  // may release multiple threads anyhow!
			check(rc == 0);
		}

		UnlockEventMutex();	
	}

	virtual void Reset() override
	{
		ResetForStats();

		check(bInitialized);
		LockEventMutex();
		Triggered = TRIGGERED_NONE;
		UnlockEventMutex();	
	}

	virtual bool Wait( uint32 WaitTime = (uint32)-1, const bool bIgnoreThreadIdleStats = false ) override;
};
