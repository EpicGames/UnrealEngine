// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_UNSUPPORTED - Unsupported platform

#include "CoreTypes.h"
#include "Misc/Timespan.h"
#include "Misc/AssertionMacros.h"
#include <pthread.h>
#include <errno.h>
#include <semaphore.h>

class FPThreadSemaphore
{
public:
	UE_NONCOPYABLE(FPThreadSemaphore);

	FPThreadSemaphore(int32 InitialCount, int32 MaxCount)
	{
		int Res = sem_init(&Semaphore, /*pshared = */ 0, InitialCount);
		checkfSlow(Res == 0, TEXT("Failed to create semaphore (%d:%d): %d"), InitialCount, MaxCount, errno);
	}

	~FPThreadSemaphore()
	{
		int Res = sem_destroy(&Semaphore);
		checkfSlow(Res == 0, TEXT("Error destroying semaphore: %d"), errno);
	}

	void Acquire()
	{
		int Res = sem_wait(&Semaphore);
		checkfSlow(Res == 0, TEXT("Acquiring semaphore failed: %d"), errno);
	}

	bool TryAcquire(FTimespan Timeout = FTimespan::Zero())
	{
		timespec ts;
		int Res = clock_gettime(CLOCK_REALTIME, &ts);
		checkfSlow(Res == 0, TEXT("clock_gettime failed: %d"), errno);

		ts.tv_sec += (uint64)Timeout.GetTotalSeconds();
		ts.tv_nsec += Timeout.GetFractionNano();
		Res = sem_timedwait(&Semaphore, &ts);
		checkfSlow(Res == 0 || Res == ETIMEDOUT, TEXT("sem_timedwait failed: %d"), errno);
		return Res == 0;
	}

	void Release(int32 Count = 1)
	{
		checkfSlow(Count > 0, TEXT("Releasing semaphore with Count = %d, that should be greater than 0"), Count);
		for (int i = 0; i != Count; ++i)
		{
			int Res = sem_post(&Semaphore);
			checkfSlow(Res == 0, TEXT("Releasing semaphore failed: %d"), errno);
		}
	}

private:
	sem_t Semaphore;
};

typedef FPThreadSemaphore FSemaphore;