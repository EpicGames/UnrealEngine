// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_UNSUPPORTED - Unsupported platform

#include "CoreTypes.h"

#if PLATFORM_USE_PTHREADS

#include <pthread.h>
#include <errno.h>

namespace UE
{

/**
 * A mutex that supports recursive locking.
 *
 * Prefer FRecursiveMutex.
 */
class FPThreadsRecursiveMutex final
{
public:
	FPThreadsRecursiveMutex(const FPThreadsRecursiveMutex&) = delete;
	FPThreadsRecursiveMutex& operator=(const FPThreadsRecursiveMutex&) = delete;

	inline FPThreadsRecursiveMutex()
	{
		pthread_mutexattr_t MutexAttributes;
		pthread_mutexattr_init(&MutexAttributes);
		pthread_mutexattr_settype(&MutexAttributes, PTHREAD_MUTEX_RECURSIVE);
		pthread_mutex_init(&Mutex, &MutexAttributes);
		pthread_mutexattr_destroy(&MutexAttributes);
	}

	UE_FORCEINLINE_HINT ~FPThreadsRecursiveMutex()
	{
		pthread_mutex_destroy(&Mutex);
	}

	UE_FORCEINLINE_HINT bool TryLock()
	{
		return pthread_mutex_trylock(&Mutex) == 0;
	}

	UE_FORCEINLINE_HINT void Lock()
	{
		pthread_mutex_lock(&Mutex);
	}

	UE_FORCEINLINE_HINT void Unlock()
	{
		pthread_mutex_unlock(&Mutex);
	}

private:
	pthread_mutex_t Mutex;
};

} // UE

#endif // PLATFORM_USE_PTHREADS
