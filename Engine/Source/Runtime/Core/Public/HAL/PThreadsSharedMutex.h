// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_UNSUPPORTED - Unsupported platform

#include "CoreTypes.h"

#if PLATFORM_USE_PTHREADS

#include "Misc/AssertionMacros.h"
#include <pthread.h>
#include <errno.h>

namespace UE
{

/**
 * A shared (read/write) mutex that does not support recursive locking.
 *
 * Prefer FSharedMutex.
 */
class FPThreadsSharedMutex final
{
public:
	FPThreadsSharedMutex(const FPThreadsSharedMutex&) = delete;
	FPThreadsSharedMutex& operator=(const FPThreadsSharedMutex&) = delete;

	FPThreadsSharedMutex()
	{
		int Err = pthread_rwlock_init(&Mutex, nullptr);
		checkf(Err == 0, TEXT("pthread_rwlock_init failed with error: %d"), Err);
	}

	~FPThreadsSharedMutex()
	{
		int Err = pthread_rwlock_destroy(&Mutex);
		checkf(Err == 0, TEXT("pthread_rwlock_destroy failed with error: %d"), Err);
	}

	bool TryLock()
	{
		int Err = pthread_rwlock_trywrlock(&Mutex);
		return Err == 0;
	}

	void Lock()
	{
		int Err = pthread_rwlock_wrlock(&Mutex);
		checkf(Err == 0, TEXT("pthread_rwlock_wrlock failed with error: %d"), Err);
	}

	void Unlock()
	{
		int Err = pthread_rwlock_unlock(&Mutex);
		checkf(Err == 0, TEXT("pthread_rwlock_unlock failed with error: %d"), Err);
	}

	bool TryLockShared()
	{
		int Err = pthread_rwlock_tryrdlock(&Mutex);
		return Err == 0;
	}

	void LockShared()
	{
		int Err = pthread_rwlock_rdlock(&Mutex);
		checkf(Err == 0, TEXT("pthread_rwlock_rdlock failed with error: %d"), Err);
	}

	void UnlockShared()
	{
		int Err = pthread_rwlock_unlock(&Mutex);
		checkf(Err == 0, TEXT("pthread_rwlock_unlock failed with error: %d"), Err);
	}

private:
	pthread_rwlock_t Mutex;
};

} // UE

#endif // PLATFORM_USE_PTHREADS
