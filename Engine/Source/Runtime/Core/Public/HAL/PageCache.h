// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "CoreTypes.h"
#include "Async/UniqueLock.h"
#include "HAL/MallocBinnedCommon.h"
#include "HAL/PlatformMutex.h"

class FPageCache
{
	UE::FPlatformRecursiveMutex Mutex;

	uint8* LowAddress;
	uint8* HighAddress;
	size_t PageSize;
	uint32 NumPages = 0;
	uint32 MemSize = 0;
	uint32 SweepPage = 0;
	uint32 CommittedPages = 0;
	uint32 DecommittedPages = 0;
	uint32 PendingDecommittedPages = 0;

	uint64 CommitHits = 0;
	uint64 CommitMisses = 0;

	FBitTree CurrentlyCommitted;
	FBitTree NotPendingDecommit;

	uint32 AddrToPageIndex(void* Addr) const
	{
		uint32 PageIndex = (((uint8*)Addr) - LowAddress) / PageSize;
		check(PageIndex < NumPages && (uint8*)Addr >= LowAddress);
		return PageIndex;
	}

	void* PageIndexToAddr(uint32 PageIndex) const
	{
		check(PageIndex < NumPages);
		return (void*)(LowAddress + size_t(PageIndex) * PageSize);
	}

	bool IsCommitted(void* Addr) const
	{
		uint32 PageIndex = AddrToPageIndex(Addr);
		bool bCommitted = CurrentlyCommitted.IsAllocated(PageIndex);
		check(bCommitted || NotPendingDecommit.IsAllocated(PageIndex));  // can't be decommited and pending decommit
		return bCommitted;
	}

	bool IsPendingDecommit(void* Addr) const
	{
		uint32 PageIndex = AddrToPageIndex(Addr);
		bool bPendingDecommit = !NotPendingDecommit.IsAllocated(PageIndex);
		check(!bPendingDecommit || CurrentlyCommitted.IsAllocated(PageIndex));  // can't be decommited and pending decommit
		return bPendingDecommit;
	}

	void Commit(void* Addr)
	{
		uint32 PageIndex = AddrToPageIndex(Addr);
		CurrentlyCommitted.AllocBit(PageIndex);
	}
	void Decommit(void* Addr)
	{
		uint32 PageIndex = AddrToPageIndex(Addr);
		CurrentlyCommitted.FreeBit(PageIndex);
	}
	void MarkPendingDecommit(void* Addr)
	{
		uint32 PageIndex = AddrToPageIndex(Addr);
		check(IsCommitted(Addr));
		NotPendingDecommit.FreeBit(PageIndex);
	}
	void UnMarkPendingDecommit(void* Addr)
	{
		uint32 PageIndex = AddrToPageIndex(Addr);
		check(IsCommitted(Addr));
		NotPendingDecommit.AllocBit(PageIndex);
	}

public:

	FPageCache(void *InLowAdress, void* InHighAddress, size_t InPageSize)
		: LowAddress((uint8*)InLowAdress)
		, HighAddress((uint8*)InHighAddress)
		, PageSize(InPageSize)
	{
	}

	uint32 GetMemoryRequirements()
	{
		check(PageSize && LowAddress && HighAddress > LowAddress);
		NumPages = (HighAddress - LowAddress) / PageSize;
		MemSize = FBitTree::GetMemoryRequirements(NumPages);
		return MemSize * 2;
	}

	void InitPageCache(void* Memory)
	{
		check(NumPages && MemSize);
		UE::TUniqueLock Lock(Mutex);
		DecommittedPages = NumPages;
		CurrentlyCommitted.FBitTreeInit(NumPages, Memory, MemSize, false);
		NotPendingDecommit.FBitTreeInit(NumPages, (uint8*)Memory + MemSize, MemSize, true);
	}

	size_t MarkForPendingDecommit(void *InAddr, size_t Size)
	{
		check(Size > 0 && IsAligned(Size, PageSize));
		uint32 StartPage = AddrToPageIndex(InAddr);
		uint32 EndPage = StartPage + Size / PageSize;

		uint32 NumFound = 0;

		UE::TUniqueLock Lock(Mutex);
		// this loop could be accelerated by using the hierarchical info in the bit tree 
		for (uint32 Index = StartPage; Index < EndPage; Index++)
		{
			if (CurrentlyCommitted.IsAllocated(Index) && NotPendingDecommit.IsAllocated(Index))
			{
				NumFound++;
				PendingDecommittedPages++;
				NotPendingDecommit.FreeBit(Index);
			}
		}
		return size_t(NumFound) * PageSize;
	}

	template<typename T>
	size_t Commit(void *InAddr, size_t Size, size_t& OutUnPending, T&& CommitFunction)
	{
		check(Size > 0 && IsAligned(Size, PageSize));
		uint32 StartPage = AddrToPageIndex(InAddr);
		uint32 EndPage = StartPage + Size / PageSize;

		uint32 LastCommitPage = MAX_uint32;
		uint32 StartCommitPage = MAX_uint32;
		UE::TUniqueLock Lock(Mutex);
		// this loop could be accelerated by using the hierarchical info in the bit tree 
		uint32 NumFound = 0;
		uint32 NumUnPending = 0;
		for (uint32 Index = StartPage; Index < EndPage; Index++)
		{
			if (CurrentlyCommitted.IsAllocated(Index))
			{
				if (!NotPendingDecommit.IsAllocated(Index))
				{
					check(PendingDecommittedPages);
					PendingDecommittedPages--;
					NumUnPending++;
					NotPendingDecommit.AllocBit(Index);
				}
			}
			else
			{
				NumFound++;
				CommittedPages++;
				check(DecommittedPages);
				DecommittedPages--;
				check(NotPendingDecommit.IsAllocated(Index));
				CurrentlyCommitted.AllocBit(Index);
				if (StartCommitPage == MAX_uint32 || LastCommitPage + 1 != Index)
				{
					if (StartCommitPage != MAX_uint32)
					{
						CommitFunction(PageIndexToAddr(StartCommitPage), (1 + LastCommitPage - StartCommitPage) * PageSize);
					}
					StartCommitPage = Index;
				}
				LastCommitPage = Index;
			}
		}
		if (StartCommitPage != MAX_uint32)
		{
			CommitFunction(PageIndexToAddr(StartCommitPage), (1 + LastCommitPage - StartCommitPage) * PageSize);
		}
		CommitHits += NumUnPending;
		CommitMisses += NumFound;
		OutUnPending = size_t(NumUnPending) * PageSize;
		return size_t(NumFound) * PageSize;
	}
	template<typename T>
	size_t ForceDecommit(void *InAddr, size_t Size, size_t& OutUnPending, T&& DecommitFunction)
	{
		check(Size > 0 && IsAligned(Size, PageSize));
		uint32 StartPage = AddrToPageIndex(InAddr);
		uint32 EndPage = StartPage + Size / PageSize;

		uint32 LastCommitPage = MAX_uint32;
		uint32 StartCommitPage = MAX_uint32;
		UE::TUniqueLock Lock(Mutex);
		// this loop could be accelerated by using the hierarchical info in the bit tree 
		uint32 NumFound = 0;
		uint32 NumUnPending = 0;
		for (uint32 Index = StartPage; Index < EndPage; Index++)
		{
			if (CurrentlyCommitted.IsAllocated(Index))
			{
				if (!NotPendingDecommit.IsAllocated(Index))
				{
					check(PendingDecommittedPages);
					PendingDecommittedPages--;
					NotPendingDecommit.AllocBit(Index);
					NumUnPending++;
				}
				NumFound++;
				check(CommittedPages);
				CommittedPages--;
				DecommittedPages++;
				CurrentlyCommitted.FreeBit(Index);
				if (StartCommitPage == MAX_uint32 || LastCommitPage + 1 != Index)
				{
					if (StartCommitPage != MAX_uint32)
					{
						DecommitFunction(PageIndexToAddr(StartCommitPage), (1 + LastCommitPage - StartCommitPage) * PageSize);
					}
					StartCommitPage = Index;
				}
				LastCommitPage = Index;
			}
		}
		if (StartCommitPage != MAX_uint32)
		{
			DecommitFunction(PageIndexToAddr(StartCommitPage), (1 + LastCommitPage - StartCommitPage) * PageSize);
		}
		OutUnPending = size_t(NumUnPending) * PageSize;
		return size_t(NumFound) * PageSize;
	}
	template<typename T>
	size_t DecommitPending(size_t Size, T&& DecommitFunction)
	{
		if (!NumPages)
		{
			return 0; // this page cache was never really set up, nothing to sweep
		}
		check(Size > 0 && IsAligned(Size, PageSize));
		uint32 NumNeed = Size / PageSize;

		uint32 LastDecommitPage = MAX_uint32;
		uint32 StartDecommitPage = MAX_uint32;
		uint32 NumFound = 0;
		UE::TUniqueLock Lock(Mutex);

		while (NumFound < NumNeed)
		{
			check(SweepPage < NumPages);
			uint32 Index = NotPendingDecommit.NextAllocBit(SweepPage);
			if (Index == MAX_uint32)
			{
				SweepPage = 0;
				break;
			}
			check(CurrentlyCommitted.IsAllocated(Index) && !NotPendingDecommit.IsAllocated(Index));
			check(CommittedPages);
			CommittedPages--;
			DecommittedPages++;
			check(PendingDecommittedPages);
			PendingDecommittedPages--;
			NumFound++;
			CurrentlyCommitted.FreeBit(Index);
			NotPendingDecommit.AllocBit(Index);
			if (StartDecommitPage == MAX_uint32 || LastDecommitPage + 1 != Index)
			{
				if (StartDecommitPage != MAX_uint32)
				{
					DecommitFunction(PageIndexToAddr(StartDecommitPage), (1 + LastDecommitPage - StartDecommitPage) * PageSize);
				}
				StartDecommitPage = Index;
			}
			LastDecommitPage = Index;
			SweepPage = Index + 1;
			if (SweepPage >= NumPages)
			{
				SweepPage = 0;
				break;
			}
		}
		if (StartDecommitPage != MAX_uint32)
		{
			DecommitFunction(PageIndexToAddr(StartDecommitPage), (1 + LastDecommitPage - StartDecommitPage) * PageSize);
		}
		return size_t(NumFound) * PageSize;
	}
	template<typename T>
	size_t TryDecommitPending(size_t Size, T&& DecommitFunction)
	{
		if (!NumPages)
		{
			return 0; // this page cache was never really set up, nothing to sweep
		}
		check(Size > 0 && IsAligned(Size, PageSize));
		uint32 NumNeed = Size / PageSize;

		uint32 LastDecommitPage = MAX_uint32;
		uint32 StartDecommitPage = MAX_uint32;
		uint32 NumFound = 0;
		UE::TUniqueLock Lock(Mutex);

		auto DecommitRange = [&]()
		{
			uint32 DecommitPageCount = 1 + LastDecommitPage - StartDecommitPage;

			if (DecommitFunction(PageIndexToAddr(StartDecommitPage), DecommitPageCount * PageSize))
			{
				CommittedPages -= DecommitPageCount;
				DecommittedPages += DecommitPageCount;
				check(PendingDecommittedPages >= DecommitPageCount);
				PendingDecommittedPages -= DecommitPageCount;
				NumFound += DecommitPageCount;

				uint32 EndBitIndex = StartDecommitPage + DecommitPageCount;
				for (uint32 BitIndex = StartDecommitPage; BitIndex < EndBitIndex; BitIndex++)
				{
					CurrentlyCommitted.FreeBit(BitIndex);
					NotPendingDecommit.AllocBit(BitIndex);
				}
			}
		};
		while (NumFound < NumNeed)
		{
			check(SweepPage < NumPages);
			uint32 Index = NotPendingDecommit.NextAllocBit(SweepPage);
			if (Index == MAX_uint32)
			{
				SweepPage = 0;
				break;
			}
			check(CurrentlyCommitted.IsAllocated(Index) && !NotPendingDecommit.IsAllocated(Index));
			check(CommittedPages);

			if (StartDecommitPage == MAX_uint32 || LastDecommitPage + 1 != Index)
			{
				if (StartDecommitPage != MAX_uint32)
				{
					DecommitRange();
				}
				StartDecommitPage = Index;
			}
			LastDecommitPage = Index;
			SweepPage = Index + 1;
			if (SweepPage >= NumPages)
			{
				SweepPage = 0;
				break;
			}
		}
		if (StartDecommitPage != MAX_uint32)
		{
			DecommitRange();
		}
		return size_t(NumFound) * PageSize;
	}
	size_t GetFreeableMemory()
	{
		UE::TUniqueLock Lock(Mutex);
		return size_t(PendingDecommittedPages) * PageSize;
	}
	float GetHitRate()
	{
		UE::TUniqueLock Lock(Mutex);
		return 100.0f * float(CommitHits) / float(CommitHits + CommitMisses + 1); // +1 to avoid divide by zero
	}
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#	include "HAL/CriticalSection.h"
#endif
