// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreGlobals.h"
#include "Modules/Boilerplate/ModuleBoilerplate.h"

UE_VISUALIZERS_HELPERS
REPLACEMENT_OPERATOR_NEW_AND_DELETE
UE_DEFINE_FMEMORY_WRAPPERS

#if !IS_MONOLITHIC
extern "C" DLLEXPORT void ThisIsAnUnrealEngineModule() {}
#endif
