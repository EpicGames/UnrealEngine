// Copyright Epic Games, Inc. All Rights Reserved.
// IWYU pragma: begin_exports
#pragma once

#include "Misc/Build.h"
#include "Misc/LargeWorldCoordinates.h"
#include <type_traits>

// define all other platforms to be zero
//@port Define the platform here to be zero when compiling for other platforms
#if !defined(PLATFORM_WINDOWS)
	#define PLATFORM_WINDOWS 0
#endif
#if !defined(PLATFORM_MAC)
	#define PLATFORM_MAC 0
	// If PLATFORM_MAC is defined these will be set appropriately in
	// MacPlatform.h
	#define PLATFORM_MAC_X86 0
	#define PLATFORM_MAC_ARM64 0
#endif
#if !defined(PLATFORM_IOS)
	#define PLATFORM_IOS 0
#endif
#if !defined(UE_PLATFORM_IOS_ONLY)
	#define UE_PLATFORM_IOS_ONLY 0
#endif
#if !defined(PLATFORM_TVOS)
	#define PLATFORM_TVOS 0
#endif
#if !defined(PLATFORM_VISIONOS)
	#define PLATFORM_VISIONOS 0
#endif
#if !defined(PLATFORM_ANDROID)
	#define PLATFORM_ANDROID 0
#endif
#if !defined(PLATFORM_ANDROID_ARM)
	#define PLATFORM_ANDROID_ARM 0
#endif
#if !defined(PLATFORM_ANDROID_ARM64)
	#define PLATFORM_ANDROID_ARM64 0
#endif
#if !defined(PLATFORM_ANDROID_X86)
	#define PLATFORM_ANDROID_X86 0
#endif
#if !defined(PLATFORM_ANDROID_X64)
	#define PLATFORM_ANDROID_X64 0
#endif
#if !defined(PLATFORM_APPLE)
	#define PLATFORM_APPLE 0
#endif
#if !defined(PLATFORM_LINUX)
	#define PLATFORM_LINUX 0
#endif
#if !defined(PLATFORM_LINUXARM64)
	#define PLATFORM_LINUXARM64 0
#endif
#if !defined(PLATFORM_SWITCH)
	#define PLATFORM_SWITCH 0
#endif
#if !defined(PLATFORM_FREEBSD)
	#define PLATFORM_FREEBSD 0
#endif
#if !defined(PLATFORM_UNIX)
	#define PLATFORM_UNIX 0
#endif
#if !defined(PLATFORM_MICROSOFT)
	#define PLATFORM_MICROSOFT 0
#endif

#if !defined(UE_PLATFORM_WINDOWS)
	#define UE_PLATFORM_WINDOWS 0
#endif
#if !defined(UE_PLATFORM_MAC)
	#define UE_PLATFORM_MAC 0
#endif
#if !defined(UE_PLATFORM_IOS)
	#define UE_PLATFORM_IOS 0
#endif
#if !defined(UE_PLATFORM_TVOS)
	#define UE_PLATFORM_TVOS 0
#endif
#if !defined(UE_PLATFORM_ANDROID)
	#define UE_PLATFORM_ANDROID 0
#endif
#if !defined(UE_PLATFORM_APPLE)
	#define UE_PLATFORM_APPLE 0
#endif
#if !defined(UE_PLATFORM_LINUX)
	#define UE_PLATFORM_LINUX 0
#endif
#if !defined(UE_PLATFORM_UNIX)
	#define UE_PLATFORM_UNIX 0
#endif
#if !defined(UE_PLATFORM_MICROSOFT)
	#define UE_PLATFORM_MICROSOFT 0
#endif

// Platform specific compiler pre-setup.
#include "PreprocessorHelpers.h"
#include COMPILED_PLATFORM_HEADER(PlatformCompilerPreSetup.h)

// Generic compiler pre-setup.
#include "GenericPlatform/GenericPlatformCompilerPreSetup.h"

// Whether the CPU is x86/x64 (i.e. both 32 and 64-bit variants)
#ifndef PLATFORM_CPU_X86_FAMILY
	#if (defined(_M_IX86) || defined(__i386__) || defined(_M_X64) || defined(__amd64__) || defined(__x86_64__)) && !defined(_M_ARM64EC)
		#define PLATFORM_CPU_X86_FAMILY	1
	#else
		#define PLATFORM_CPU_X86_FAMILY	0
	#endif
#endif

// Whether the CPU is AArch32/AArch64 (i.e. both 32 and 64-bit variants)
#ifndef PLATFORM_CPU_ARM_FAMILY
	#if (defined(__arm__) || defined(_M_ARM) || defined(__aarch64__) || defined(_M_ARM64) || defined(_M_ARM64EC))
		#define PLATFORM_CPU_ARM_FAMILY	1
	#else
		#define PLATFORM_CPU_ARM_FAMILY	0
	#endif
#endif

// Whether the platform is Arm64ec. Normal Windows Arm64 can be detected with PLATFORM_CPU_ARM_FAMILY && PLATFORM_WINDOWS
#if defined(_M_ARM64EC)
	#define PLATFORM_WINDOWS_ARM64EC PLATFORM_WINDOWS
#else
	#define PLATFORM_WINDOWS_ARM64EC 0
#endif

#if PLATFORM_APPLE
	#include <stddef.h> // needed for size_t
#endif

#include "GenericPlatform/GenericPlatform.h"

//------------------------------------------------------------------
// Setup macros for compiler detection
//------------------------------------------------------------------
#ifndef PLATFORM_COMPILER_CLANG
#if defined(__clang__)
#define PLATFORM_COMPILER_CLANG			1
#else
#define PLATFORM_COMPILER_CLANG			0
#endif // defined(__clang__)
#endif

#ifndef PLATFORM_COMPILER_MSVC
#if defined(_MSC_VER) && !PLATFORM_COMPILER_CLANG
#define PLATFORM_COMPILER_MSVC			1
#else
#define PLATFORM_COMPILER_MSVC			0
#endif // defined(_MSC_VER) ...
#endif

//------------------------------------------------------------------
// Setup macros for static code analysis
//------------------------------------------------------------------
#if PLATFORM_COMPILER_CLANG
	#include "Clang/ClangPlatformCodeAnalysis.h"
#elif PLATFORM_MICROSOFT
	#include "Microsoft/MicrosoftPlatformCodeAnalysis.h"
#endif

#ifndef USING_ADDRESS_SANITISER
	#define USING_ADDRESS_SANITISER 0
#endif

#ifndef USING_HW_ADDRESS_SANITISER
	#define USING_HW_ADDRESS_SANITISER 0
#endif

#ifndef USING_THREAD_SANITISER
	#define USING_THREAD_SANITISER 0
#endif

#ifndef USING_UNDEFINED_BEHAVIOR_SANITISER
	#define USING_UNDEFINED_BEHAVIOR_SANITISER 0
#endif

#ifndef USING_MEMORY_SANITISER
	#define USING_MEMORY_SANITISER 0
#endif

#ifndef PLATFORM_HAS_ASAN_INCLUDE
	#define PLATFORM_HAS_ASAN_INCLUDE __has_include(<sanitizer/asan_interface.h>)
#endif

//---------------------------------------------------------
// Include main platform setup header (XXX/XXXPlatform.h)
//---------------------------------------------------------

#include COMPILED_PLATFORM_HEADER(Platform.h)

//------------------------------------------------------------------
// Finalize define setup
//------------------------------------------------------------------

// Base defines, must define these for the platform, there are no defaults
#ifndef PLATFORM_DESKTOP
	#error "PLATFORM_DESKTOP must be defined"
#endif

// Base defines, these have defaults
#ifndef PLATFORM_LITTLE_ENDIAN
	#define PLATFORM_LITTLE_ENDIAN				0
#endif
#ifndef PLATFORM_SUPPORTS_UNALIGNED_LOADS
	#define PLATFORM_SUPPORTS_UNALIGNED_LOADS	0
#endif
#ifndef PLATFORM_EXCEPTIONS_DISABLED
	#define PLATFORM_EXCEPTIONS_DISABLED		!PLATFORM_DESKTOP
#endif
#ifndef PLATFORM_SEH_EXCEPTIONS_DISABLED
	#define PLATFORM_SEH_EXCEPTIONS_DISABLED	0
#endif
#ifndef PLATFORM_SUPPORTS_PRAGMA_PACK
	#define PLATFORM_SUPPORTS_PRAGMA_PACK		0
#endif

// Defines for the availibility of the various levels of vector intrinsics.
// These may be set from UnrealBuildTool, otherwise each platform-specific platform.h is expected to set them appropriately.
#ifndef PLATFORM_ENABLE_VECTORINTRINSICS
	#define PLATFORM_ENABLE_VECTORINTRINSICS	0
#endif

// Defines if the platform can handle setting vectorized exception
// and rounding modes
#ifndef PLATFORM_SUPPORTS_VECTOR_CONTROL_REGISTERS
	#define PLATFORM_SUPPORTS_VECTOR_CONTROL_REGISTERS 1
#endif

#ifndef PLATFORM_USE_SSE2_FOR_THREAD_YIELD
	#define PLATFORM_USE_SSE2_FOR_THREAD_YIELD	PLATFORM_CPU_X86_FAMILY
#endif

// If PLATFORM_MAYBE_HAS_### is 1, then ### intrinsics are compilable.
// This does not guarantee that the intrinsics are runnable on all instances of the platform however; a runtime check such as cpuid may be required to confirm availability.
// If PLATFORM_ALWAYS_HAS_### is 1, then ## intrinsics will compile and run on all instances of the platform.  PLATFORM_ALWAYS_HAS_### == 1 implies PLATFORM_MAYBE_HAS_### == 1.

// UE5.2+ requires SSE4.2.
#ifndef PLATFORM_MAYBE_HAS_SSE4_1
	#define PLATFORM_MAYBE_HAS_SSE4_1			PLATFORM_CPU_X86_FAMILY
#endif
#ifndef PLATFORM_ALWAYS_HAS_SSE4_1
	#define PLATFORM_ALWAYS_HAS_SSE4_1			PLATFORM_CPU_X86_FAMILY
#endif
#ifndef PLATFORM_ALWAYS_HAS_SSE4_2
	#define PLATFORM_ALWAYS_HAS_SSE4_2			PLATFORM_CPU_X86_FAMILY
#endif

#ifndef PLATFORM_MAYBE_HAS_AVX
	#define PLATFORM_MAYBE_HAS_AVX				0
#endif
#ifndef PLATFORM_ALWAYS_HAS_AVX
	#define PLATFORM_ALWAYS_HAS_AVX				0
#endif
#ifndef PLATFORM_ALWAYS_HAS_AVX_2
	#define PLATFORM_ALWAYS_HAS_AVX_2			0
#endif
#ifndef PLATFORM_ALWAYS_HAS_FMA3
	#define PLATFORM_ALWAYS_HAS_FMA3			0
#endif
#ifndef PLATFORM_ALWAYS_HAS_AESNI
	#define PLATFORM_ALWAYS_HAS_AESNI			0
#endif
#ifndef PLATFORM_ALWAYS_HAS_SHA
	#define PLATFORM_ALWAYS_HAS_SHA				0
#endif


#ifndef PLATFORM_HAS_CPUID
	#if defined(_M_IX86) || defined(__i386__) || defined(_M_X64) || defined(__x86_64__) || defined (__amd64__)
		#define PLATFORM_HAS_CPUID				1
	#else
		#define PLATFORM_HAS_CPUID				0
	#endif
#endif
#define PLATFORM_ENABLE_POPCNT_INTRINSIC UE_DEPRECATED_MACRO(5.8, "PLATFORM_ENABLE_POPCNT_INTRINSIC has been deprecated and should be replaced with 1.") 1
#ifndef PLATFORM_ENABLE_VECTORINTRINSICS_NEON
	#define PLATFORM_ENABLE_VECTORINTRINSICS_NEON	0
#endif
#ifndef UE_VALIDATE_FORMAT_STRINGS
	#define UE_VALIDATE_FORMAT_STRINGS 0
#endif
// Does vswprintf want %ls instead of %s when displaying wide strings?
#ifndef PLATFORM_USE_LS_SPEC_FOR_WIDECHAR
	#define PLATFORM_USE_LS_SPEC_FOR_WIDECHAR	1
#endif
// Does vsprintf want %s instead of %hs when displaying narrow strings?
#ifndef PLATFORM_USE_S_SPEC_FOR_NARROWCHAR_IN_VSPRINTF
	#define PLATFORM_USE_S_SPEC_FOR_NARROWCHAR_IN_VSPRINTF	0
#endif
#ifndef PLATFORM_USE_SYSTEM_VSWPRINTF
	#define PLATFORM_USE_SYSTEM_VSWPRINTF		1
#endif
#ifndef PLATFORM_COMPILER_DISTINGUISHES_INT_AND_LONG
	#define PLATFORM_COMPILER_DISTINGUISHES_INT_AND_LONG			0
#endif
#ifndef PLATFORM_COMPILER_HAS_GENERIC_KEYWORD
	#define PLATFORM_COMPILER_HAS_GENERIC_KEYWORD	0
#endif
#ifndef PLATFORM_COMPILER_COMMON_LANGUAGE_RUNTIME_COMPILATION
	#define PLATFORM_COMPILER_COMMON_LANGUAGE_RUNTIME_COMPILATION 0
#endif
#ifndef PLATFORM_COMPILER_HAS_TCHAR_WMAIN
	#define PLATFORM_COMPILER_HAS_TCHAR_WMAIN 0
#endif
#ifndef PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	#define PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS (__cplusplus >= 202002L)
#endif
// Feature test macro for constexpr __builtin_FILE() and __builtin_LINE()
#define PLATFORM_COMPILER_SUPPORTS_CONSTEXPR_BUILTIN_FILE_AND_LINE UE_DEPRECATED_MACRO(5.7, "PLATFORM_COMPILER_SUPPORTS_CONSTEXPR_BUILTIN_FILE_AND_LINE has been deprecated and should be replaced with 1.") 1
#ifndef PLATFORM_COMPILER_SUPPORTS_BUILTIN_BITCAST
	#error PLATFORM_COMPILER_SUPPORTS_BUILTIN_BITCAST should have been defined by now
#endif
#define PLATFORM_COMPILER_SUPPORTS_CONSTRAINED_DESTRUCTORS UE_DEPRECATED_MACRO(5.8, "PLATFORM_COMPILER_SUPPORTS_CONSTRAINED_DESTRUCTORS has been deprecated and should be replaced with 1.") 1
#define PLATFORM_TCHAR_IS_4_BYTES UE_DEPRECATED_MACRO(5.8, "PLATFORM_TCHAR_IS_4_BYTES has been deprecated and should be replaced with 0.") 0
#ifndef PLATFORM_WCHAR_IS_4_BYTES
	#define PLATFORM_WCHAR_IS_4_BYTES			0
#endif
// The PLATFORM_TCHAR_IS_CHAR16 macro really represents PLATFORM_WIDECHAR_IS_CHAR16 - it should be deprecated
#ifndef PLATFORM_TCHAR_IS_CHAR16
	#define PLATFORM_TCHAR_IS_CHAR16			0
#endif
#define PLATFORM_WIDECHAR_IS_CHAR16 PLATFORM_TCHAR_IS_CHAR16
#ifndef PLATFORM_TCHAR_IS_UTF8CHAR
	#define PLATFORM_TCHAR_IS_UTF8CHAR			USE_UTF8_TCHARS
#endif
#ifndef PLATFORM_UCS2CHAR_IS_UTF16CHAR
	// Currently true, but we don't want it to be true
	#define PLATFORM_UCS2CHAR_IS_UTF16CHAR		1
#endif
#ifndef PLATFORM_HAS_BSD_TIME
	#define PLATFORM_HAS_BSD_TIME				1
#endif
#ifndef PLATFORM_HAS_BSD_THREAD_CPUTIME
	#define PLATFORM_HAS_BSD_THREAD_CPUTIME		0
#endif
#ifndef PLATFORM_HAS_BSD_SOCKETS
	#define PLATFORM_HAS_BSD_SOCKETS			1
#endif
#ifndef PLATFORM_HAS_BSD_IPV6_SOCKETS
	#define PLATFORM_HAS_BSD_IPV6_SOCKETS			0
#endif
#ifndef PLATFORM_SUPPORTS_UDP_MULTICAST_GROUP
	#define PLATFORM_SUPPORTS_UDP_MULTICAST_GROUP	1
#endif
#ifndef PLATFORM_USE_PTHREADS
	#define PLATFORM_USE_PTHREADS				1
#endif
#ifndef PLATFORM_HAS_MULTITHREADED_PREMAIN
	/**
	 * If true, the platform uses multiple threads in c++ static initialization before main is called, and systems
	 * accessible from multiple threads during premain static initialization must handle multithreaded synchronization.
	 */
	#define PLATFORM_HAS_MULTITHREADED_PREMAIN 0
#endif
#ifndef PLATFORM_MAX_FILEPATH_LENGTH_DEPRECATED
	#define PLATFORM_MAX_FILEPATH_LENGTH_DEPRECATED		128			// Deprecated - prefer FPlatformMisc::GetMaxPathLength() instead.
#endif
#ifndef PLATFORM_SUPPORTS_TEXTURE_STREAMING
	#define PLATFORM_SUPPORTS_TEXTURE_STREAMING	1
#endif
#ifndef PLATFORM_SUPPORTS_VIRTUAL_TEXTURES
	#define PLATFORM_SUPPORTS_VIRTUAL_TEXTURES		0
#endif
#ifndef PLATFORM_SUPPORTS_VARIABLE_RATE_SHADING
	#define PLATFORM_SUPPORTS_VARIABLE_RATE_SHADING		0
#endif
#ifndef PLATFORM_REQUIRES_FILESERVER
	#define PLATFORM_REQUIRES_FILESERVER		0
#endif
#ifndef PLATFORM_SUPPORTS_MULTITHREADED_GC
	#define PLATFORM_SUPPORTS_MULTITHREADED_GC	1
#endif
#ifndef PLATFORM_HAS_BSD_SOCKET_FEATURE_IOCTL
	#define PLATFORM_HAS_BSD_SOCKET_FEATURE_IOCTL	1
#endif
#ifndef PLATFORM_HAS_BSD_SOCKET_FEATURE_POLL
	#define PLATFORM_HAS_BSD_SOCKET_FEATURE_POLL	0
#endif
#ifndef PLATFORM_HAS_BSD_SOCKET_FEATURE_SELECT
	#define PLATFORM_HAS_BSD_SOCKET_FEATURE_SELECT	1
#endif
#ifndef PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
	#define PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS	0
#endif
#ifndef PLATFORM_HAS_BSD_SOCKET_FEATURE_GETHOSTNAME
	#define PLATFORM_HAS_BSD_SOCKET_FEATURE_GETHOSTNAME	1
#endif
#ifndef PLATFORM_HAS_BSD_SOCKET_FEATURE_GETADDRINFO
	#define PLATFORM_HAS_BSD_SOCKET_FEATURE_GETADDRINFO	1
#endif
#ifndef PLATFORM_HAS_BSD_SOCKET_FEATURE_GETNAMEINFO
	#define PLATFORM_HAS_BSD_SOCKET_FEATURE_GETNAMEINFO 1
#endif
#ifndef PLATFORM_HAS_BSD_SOCKET_FEATURE_CLOSE_ON_EXEC
	#define PLATFORM_HAS_BSD_SOCKET_FEATURE_CLOSE_ON_EXEC	0
#endif
#ifndef PLATFORM_HAS_BSD_SOCKET_FEATURE_MSG_DONTWAIT
	#define PLATFORM_HAS_BSD_SOCKET_FEATURE_MSG_DONTWAIT	0
#endif
#ifndef PLATFORM_HAS_BSD_SOCKET_FEATURE_RECVMMSG
	#define PLATFORM_HAS_BSD_SOCKET_FEATURE_RECVMMSG	0
#endif
#ifndef PLATFORM_HAS_BSD_SOCKET_FEATURE_TIMESTAMP
	#define PLATFORM_HAS_BSD_SOCKET_FEATURE_TIMESTAMP 0
#endif
#ifndef PLATFORM_HAS_BSD_SOCKET_FEATURE_NODELAY
	#define PLATFORM_HAS_BSD_SOCKET_FEATURE_NODELAY	1
#endif
#ifndef PLATFORM_HAS_NO_EPROCLIM
	#define PLATFORM_HAS_NO_EPROCLIM			0
#endif
#ifndef PLATFORM_USES_MICROSOFT_LIBC_FUNCTIONS
	#define PLATFORM_USES_MICROSOFT_LIBC_FUNCTIONS 0
#endif

#ifndef PLATFORM_USES_GLES
	#define PLATFORM_USES_GLES					0
#endif

#ifndef PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	#define PLATFORM_SUPPORTS_GEOMETRY_SHADERS		1
#endif

#ifndef PLATFORM_SUPPORTS_MESH_SHADERS
	#define PLATFORM_SUPPORTS_MESH_SHADERS 0
#endif

#ifndef PLATFORM_SUPPORTS_WORKGRAPH_SHADERS
	#define PLATFORM_SUPPORTS_WORKGRAPH_SHADERS 0
#endif

#ifndef PLATFORM_SUPPORTS_BINDLESS_RENDERING
	#define PLATFORM_SUPPORTS_BINDLESS_RENDERING 0
#endif

#ifndef PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS 
	#define PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS 0
#endif

#ifndef PLATFORM_SUPPORT_SPLIT_PLAYERS_RT
	#define PLATFORM_SUPPORT_SPLIT_PLAYERS_RT 0
#endif

#ifndef PLATFORM_BUILTIN_VERTEX_HALF_FLOAT
	#define PLATFORM_BUILTIN_VERTEX_HALF_FLOAT	1
#endif

#ifndef PLATFORM_SUPPORTS_TBB
	#define PLATFORM_SUPPORTS_TBB 0
#endif

#ifndef PLATFORM_SUPPORTS_BORDERLESS_WINDOW
	#define PLATFORM_SUPPORTS_BORDERLESS_WINDOW		0
#endif

#ifndef PLATFORM_MAX_CACHED_SYNC_FILE_HANDLES_PER_GENERIC_ASYNC_FILE_HANDLE
	#define PLATFORM_MAX_CACHED_SYNC_FILE_HANDLES_PER_GENERIC_ASYNC_FILE_HANDLE 4
#endif

#ifndef PLATFORM_FORCE_SINGLE_SYNC_FILE_HANDLE_PER_GENERIC_ASYNC_FILE_HANDLE
	#define PLATFORM_FORCE_SINGLE_SYNC_FILE_HANDLE_PER_GENERIC_ASYNC_FILE_HANDLE 0
#endif

#ifndef PLATFORM_SUPPORTS_JEMALLOC
	#define PLATFORM_SUPPORTS_JEMALLOC 0
#endif

#ifndef PLATFORM_SUPPORTS_MIMALLOC
	#define PLATFORM_SUPPORTS_MIMALLOC 0
#endif

#ifndef PLATFORM_CAN_SUPPORT_EDITORONLY_DATA
	#define PLATFORM_CAN_SUPPORT_EDITORONLY_DATA 0
#endif

#ifndef PLATFORM_SUPPORTS_NAMED_PIPES
	#define PLATFORM_SUPPORTS_NAMED_PIPES		0
#endif

#ifndef PLATFORM_USES_FIXED_RHI_CLASS
	#define PLATFORM_USES_FIXED_RHI_CLASS		0
#endif

#ifndef PLATFORM_USES_FIXED_GMalloc_CLASS
	#define PLATFORM_USES_FIXED_GMalloc_CLASS		0
#endif

#ifndef PLATFORM_USES_STACKBASED_MALLOC_CRASH
	#define PLATFORM_USES_STACKBASED_MALLOC_CRASH	0
#endif

#ifndef PLATFORM_SUPPORTS_MULTIPLE_NATIVE_WINDOWS
	#define PLATFORM_SUPPORTS_MULTIPLE_NATIVE_WINDOWS	1
#endif

#ifndef PLATFORM_HAS_TOUCH_MAIN_SCREEN
	#define PLATFORM_HAS_TOUCH_MAIN_SCREEN		0
#endif

#ifndef PLATFORM_SUPPORTS_STACK_SYMBOLS
	#define PLATFORM_SUPPORTS_STACK_SYMBOLS 0
#endif

#ifndef PLATFORM_HAS_128BIT_ATOMICS
	#define PLATFORM_HAS_128BIT_ATOMICS 0
#endif

#ifndef PLATFORM_USES_ANSI_STRING_FOR_EXTERNAL_PROFILING
	#define PLATFORM_USES_ANSI_STRING_FOR_EXTERNAL_PROFILING	1
#endif

#ifndef PLATFORM_IMPLEMENTS_BeginNamedEventStatic
	#define PLATFORM_IMPLEMENTS_BeginNamedEventStatic			0
#endif

#ifndef PLATFORM_HAS_UMA
	#define PLATFORM_HAS_UMA 0
#endif

#ifndef PLATFORM_NUM_AUDIODECOMPRESSION_PRECACHE_BUFFERS
	#define PLATFORM_NUM_AUDIODECOMPRESSION_PRECACHE_BUFFERS 2
#endif

#ifndef PLATFORM_SUPPORTS_EARLY_MOVIE_PLAYBACK
	#define PLATFORM_SUPPORTS_EARLY_MOVIE_PLAYBACK			0
#endif

#ifndef PLATFORM_UI_HAS_MOBILE_SCROLLBARS
	#define PLATFORM_UI_HAS_MOBILE_SCROLLBARS					0
#endif

#ifndef PLATFORM_UI_NEEDS_TOOLTIPS
	#define PLATFORM_UI_NEEDS_TOOLTIPS							1
#endif

#ifndef PLATFORM_UI_NEEDS_FOCUS_OUTLINES
	#define PLATFORM_UI_NEEDS_FOCUS_OUTLINES					1
#endif

#ifndef PLATFORM_LIMIT_MOBILE_BONE_MATRICES
	#define PLATFORM_LIMIT_MOBILE_BONE_MATRICES					0
#endif

#ifndef PLATFORM_WEAKLY_CONSISTENT_MEMORY
	#define PLATFORM_WEAKLY_CONSISTENT_MEMORY PLATFORM_CPU_ARM_FAMILY
#endif

#ifndef PLATFORM_NEEDS_RHIRESOURCELIST
	#define PLATFORM_NEEDS_RHIRESOURCELIST 1
#endif

#ifndef PLATFORM_USE_FULL_TASK_GRAPH
	#define PLATFORM_USE_FULL_TASK_GRAPH						1
#endif

#ifndef PLATFORM_USE_ANSI_POSIX_MALLOC
	#define PLATFORM_USE_ANSI_POSIX_MALLOC						0
#endif

#ifndef PLATFORM_USES__ALIGNED_MALLOC
	#define PLATFORM_USES__ALIGNED_MALLOC						0
#endif

#ifndef PLATFORM_USE_ANSI_MEMALIGN
	#define PLATFORM_USE_ANSI_MEMALIGN							0
#endif

#ifndef PLATFORM_IS_ANSI_MALLOC_THREADSAFE
	#define PLATFORM_IS_ANSI_MALLOC_THREADSAFE					0
#endif

#ifndef PLATFORM_SUPPORTS_ASYMMETRIC_FENCES
	#define PLATFORM_SUPPORTS_ASYMMETRIC_FENCES					0
#endif

#ifndef PLATFORM_SUPPORTS_OPUS_CODEC
	#define PLATFORM_SUPPORTS_OPUS_CODEC						1
#endif

#ifndef PLATFORM_SUPPORTS_VORBIS_CODEC
	#define PLATFORM_SUPPORTS_VORBIS_CODEC						1
#endif

#ifndef PLATFORM_SUPPORTS_VORBIS_ENCODE
	#define PLATFORM_SUPPORTS_VORBIS_ENCODE						PLATFORM_SUPPORTS_VORBIS_CODEC
#endif

#ifndef PLATFORM_USE_MINIMAL_HANG_DETECTION
	#define PLATFORM_USE_MINIMAL_HANG_DETECTION					0
#endif

#ifndef PLATFORM_USE_GENERIC_STRING_IMPLEMENTATION
	#define PLATFORM_USE_GENERIC_STRING_IMPLEMENTATION			1
#endif

#ifndef PLATFORM_ALLOW_ALLOCATIONS_IN_FASYNCWRITER_SERIALIZEBUFFERTOARCHIVE
	#define	PLATFORM_ALLOW_ALLOCATIONS_IN_FASYNCWRITER_SERIALIZEBUFFERTOARCHIVE 1
#endif

// Our  LowLevelMemTracker wants to store data about an allocation in the high bits, so it assumes high bits are 0.
// Some platforms store data about an allocation of their own in the high bits. Let LLM know that on those platforms 
// it is okay to discard those bits when storing a copy of the pointer for reference.
#if !defined(PLATFORM_STORES_DATA_IN_POINTER_HIGH_BITS)
	#define PLATFORM_STORES_DATA_IN_POINTER_HIGH_BITS 0
#endif

#ifndef PLATFORM_HAS_FPlatformVirtualMemoryBlock
	#define	PLATFORM_HAS_FPlatformVirtualMemoryBlock 1
#endif

#ifndef PLATFORM_BYPASS_PAK_PRECACHE
	#define PLATFORM_BYPASS_PAK_PRECACHE 0
#endif

#ifndef PLATFORM_SUPPORTS_FLIP_TRACKING
	#define PLATFORM_SUPPORTS_FLIP_TRACKING 0
#endif

#ifndef PLATFORM_USE_SHOWFLAGS_ALWAYS_BITFIELD
	#define	PLATFORM_USE_SHOWFLAGS_ALWAYS_BITFIELD				1
#endif

#ifndef PLATFORM_USE_GENERIC_LAUNCH_IMPLEMENTATION
	#define PLATFORM_USE_GENERIC_LAUNCH_IMPLEMENTATION			0
#endif

#ifndef PLATFORM_USES_FIXED_HDR_SETTING
	#define PLATFORM_USES_FIXED_HDR_SETTING 0
#endif

#ifndef PLATFORM_USES_DYNAMIC_HDR_SETTING
	#define PLATFORM_USES_DYNAMIC_HDR_SETTING 0
#endif

#ifndef PLATFORM_PROVIDES_HDR_PAPER_WHITE
	#define PLATFORM_PROVIDES_HDR_PAPER_WHITE 0
#endif

#ifndef PLATFORM_MANAGES_HDR_SETTING
	#define PLATFORM_MANAGES_HDR_SETTING 0
#endif

#ifndef PLATFORM_SUPPORTS_COLORIZED_OUTPUT_DEVICE
	#define PLATFORM_SUPPORTS_COLORIZED_OUTPUT_DEVICE PLATFORM_DESKTOP
#endif

#ifndef PLATFORM_USE_PLATFORM_FILE_MANAGED_STORAGE_WRAPPER
	#define PLATFORM_USE_PLATFORM_FILE_MANAGED_STORAGE_WRAPPER	0
#endif

#ifndef PLATFORM_HAS_DIRECT_TEXTURE_MEMORY_ACCESS
	#define PLATFORM_HAS_DIRECT_TEXTURE_MEMORY_ACCESS 0
#endif

#ifndef PLATFORM_DIRECT_TEXTURE_MEMORY_ACCESS_LOCK_MODE 
	#define PLATFORM_DIRECT_TEXTURE_MEMORY_ACCESS_LOCK_MODE RLM_ReadOnly 
#endif

#ifndef PLATFORM_USE_REPORT_ENSURE
	#define PLATFORM_USE_REPORT_ENSURE PLATFORM_DESKTOP
#endif

#ifndef PLATFORM_USE_FALLBACK_PSO
	#define PLATFORM_USE_FALLBACK_PSO 0
#endif

#ifndef PLATFORM_SUPPORTS_PSO_PRECACHING
	#define PLATFORM_SUPPORTS_PSO_PRECACHING (!UE_SERVER)
#endif

#ifndef PLATFORM_SUPPORTS_DYNAMIC_SHADER_PRELOADING
#define PLATFORM_SUPPORTS_DYNAMIC_SHADER_PRELOADING (!UE_SERVER)
#endif

#ifndef PLATFORM_USES_UNFAIR_LOCKS
	#define PLATFORM_USES_UNFAIR_LOCKS 0
#endif

#ifndef PLATFORM_HAS_FENV_H
	#define PLATFORM_HAS_FENV_H 1
#endif

#ifndef PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
	#define PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND 0
#endif

#ifndef PLATFORM_CONSOLE_DYNAMIC_LINK
	#define PLATFORM_CONSOLE_DYNAMIC_LINK 0
#endif

#ifndef PLATFORM_MAX_UNIFORM_BUFFER_RANGE
	#define PLATFORM_MAX_UNIFORM_BUFFER_RANGE (16u*1024u)
#endif

#ifndef PLATFORM_IMPLEMENTS_BATCH_FILE_DELETE
	#define PLATFORM_IMPLEMENTS_BATCH_FILE_DELETE 0
#endif

#ifndef PLATFORM_WRITES_ARE_SLOW
	#define PLATFORM_WRITES_ARE_SLOW 0
#endif

#ifndef PLATFORM_FORCE_UE_LOCK_USAGE
	#define PLATFORM_FORCE_UE_LOCK_USAGE 0
#endif

#ifndef PLATFORM_BREAK_IS_EXPRESSION
	#define PLATFORM_BREAK_IS_EXPRESSION 0
#endif

#ifndef PLATFORM_THREAD_LOCAL_IS_UNSAFE_IN_MODULES
	#define PLATFORM_THREAD_LOCAL_IS_UNSAFE_IN_MODULES 0
#endif

// deprecated, do not use
#define PLATFORM_HAS_THREADSAFE_RHIGetRenderQueryResult	#
#define PLATFORM_SUPPORTS_RHI_THREAD #
#define PLATFORM_RHI_USES_CONTEXT_OBJECT # // deprecated, do not use; all platforms must use a context object
#define PLATFORM_SUPPORTS_PARALLEL_RHI_EXECUTE # // deprecated, do not use; see GRHISupportsParallelRHIExecute


//These are deprecated old defines that we want to make sure are not used
#define CONSOLE (#)
#define MOBILE (#)
#define PLATFORM_CONSOLE (#)

#ifdef PLATFORM_32BITS
	#error PLATFORM_32BITS should not be defined.
#endif
#ifdef PLATFORM_64BITS
	#error PLATFORM_64BITS should not be defined.
#endif
#define PLATFORM_32BITS UE_DEPRECATED_MACRO(5.8, "PLATFORM_32BITS is deprecated as UE only supports 64-bit.") 0
#define PLATFORM_64BITS UE_DEPRECATED_MACRO(5.8, "PLATFORM_64BITS is deprecated as UE only supports 64-bit.") 1

// not supported by the platform system yet or maybe ever
#define PLATFORM_VTABLE_AT_END_OF_CLASS 0

#ifndef VARARGS
	#define VARARGS									/* Functions with variable arguments */
#endif
#ifndef CDECL
	#define CDECL									/* Standard C function */
#endif
#ifndef STDCALL
	#define STDCALL									/* Standard calling convention */
#endif
#ifndef FORCEINLINE
	#define FORCEINLINE 							/* Force code to be inline */
#endif
#ifndef FORCENOINLINE
	#define FORCENOINLINE 							/* Force code to NOT be inline */
#endif
#ifndef RESTRICT
	#define RESTRICT __restrict						/* no alias hint */
#endif
#ifndef CALLSITE_FORCEINLINE
	#define CALLSITE_FORCEINLINE 					/* Force code to be inline at the callsite */
#endif
#ifndef UE_DEFINE_FORCEINLINE_HINT_TO_INLINE
	#define UE_DEFINE_FORCEINLINE_HINT_TO_INLINE 0
#endif

// A configurable alternative to FORCEINLINE than can be configured per target.
// The main use of this is for core functionality that is performance sensitive but has a large code size cost.
// This allows keeping code force inlined by default to improve performance in cases where it wouldn't normally be inlined,
// but also allow disabling it for targets where the compiler can make more informed decisions such as when using PGO.
// This ultimately should result in a better size/speed trade off in those cases.
#if UE_DEFINE_FORCEINLINE_HINT_TO_INLINE
	#define UE_FORCEINLINE_HINT inline
#else
	#define UE_FORCEINLINE_HINT FORCEINLINE
#endif

// UE_REWRITE should be placed on functions which rewrite a function call as another expression. Like a macro,
// but without using the preprocessor.
//
// Its intent is different from FORCEINLINE, which can be used for optimization reasons. UE_REWRITE should be applied
// to functions which need to exist for API reasons, but where we don't want any interactions with the user experience,
// like being forced to step through them in the debugger, or having symbols generated for them.
//
// Having a dedicated macro for this specific use case, separate from FORCEINLINE, makes it easier to reason about
// and spot when it's being misapplied. As a rule of thumb, a rewrite function should consist of a single return statement
// with at most one call to a non-rewrite function.
//
// Examples:
// - Forwarding calls.
// - Trivial getters.
// - Iterators.
// - Related overloads implemented in terms of each other, like const/non-const, lvalue/rvalue, operator==/operator!=.
// - Lvalue/rvalue overloads where one is implemented in terms of the other.
// - Cast functions like MoveTemp, Forward or ImplicitConv.
// - operator-> on smart pointers.
// - View constructors.
#ifndef UE_REWRITE
	#define UE_REWRITE UE_NODEBUG FORCEINLINE
#endif

// Use before a constructor declaration to warn when an unnamed temporary object is ignored, e.g. FScopeLock(CS); instead of FScopeLock Lock(CS);
// We can't just use UE_NODISCARD in this case because older compilers don't like [[nodiscard]] on constructors.
#if !defined(UE_NODISCARD_CTOR) && defined(__has_cpp_attribute)
	#if __has_cpp_attribute(nodiscard)
		#if (defined(_MSC_VER) && _MSC_VER >= 1924) || (defined(__clang__) && __clang_major__ >= 10)
			#define UE_NODISCARD_CTOR [[nodiscard]]
		#endif
	#endif
#endif

#ifndef UE_NODISCARD_CTOR
	#define UE_NODISCARD_CTOR
#endif

/* Use in front of initialized global variable to have link once-semantics
   When multiple definitions of the variable are encountered by the linker, the first is selected and the remainder are discarded */
#ifdef _MSC_VER
    #define UE_SELECT_ANY __declspec(selectany)
#else
    #define UE_SELECT_ANY __attribute__((selectany))
#endif

/* Macro wrapper for the consteval keyword which isn't yet present on all compilers - constexpr
   can be used as a workaround but is less strict and so may let some non-consteval code pass */
#if defined(__cpp_consteval)
	#define UE_CONSTEVAL consteval
#else
	#define UE_CONSTEVAL constexpr
#endif

/* Macro wrapper for the consteval if statement which isn't yet present on all compilers.
   Can be used in place of std::is_constant_evaluated() which isn't yet available on all platforms. */
#if defined(__cpp_if_consteval)
	#define UE_IF_CONSTEVAL if consteval
	#define UE_IF_NOT_CONSTEVAL if !consteval
#else
	#define UE_IF_CONSTEVAL if (__builtin_is_constant_evaluated())
	#define UE_IF_NOT_CONSTEVAL if (!__builtin_is_constant_evaluated())
#endif

/* Use before a class data member declaration allow it to be overlapped with other non-static data members or base class subobjects of its class. */
#if !defined(UE_NO_UNIQUE_ADDRESS) && defined(__has_cpp_attribute)
	#if __has_cpp_attribute(msvc::no_unique_address)
		#define UE_NO_UNIQUE_ADDRESS [[msvc::no_unique_address]] 
	#elif __has_cpp_attribute(no_unique_address)
		#define UE_NO_UNIQUE_ADDRESS [[no_unique_address]]
	#endif
#endif
#ifndef UE_NO_UNIQUE_ADDRESS
	#define UE_NO_UNIQUE_ADDRESS
#endif

/* Wrap a function signature in these to indicate that the function never returns nullptr */
#ifndef FUNCTION_NON_NULL_RETURN_START
	#define FUNCTION_NON_NULL_RETURN_START
#endif
#ifndef FUNCTION_NON_NULL_RETURN_END
	#define FUNCTION_NON_NULL_RETURN_END
#endif

/* Wrap lifetimebound annotations to indicate that a function argument must outlive a return value or constructed object */
#ifndef UE_LIFETIMEBOUND
	#define UE_LIFETIMEBOUND
#endif

/* Tag a function or variable to omit debug info for this symbol */
#ifndef UE_NODEBUG
	#define UE_NODEBUG
#endif

/* Annotate functions that allocate new memory to ensure the compiler can optimize them accordingly.
   The arguments to this macro specify the 1-based arguments of the annotated function that specify the size of the allocation. */
#ifndef UE_ALLOCATION_FUNCTION
	#define UE_ALLOCATION_FUNCTION(...)
#endif

/** Promise expression is true. Compiler can optimize accordingly with undefined behavior if wrong. Static analyzers understand this.  */
#ifndef UE_ASSUME
	#if defined(__clang__)
		#define UE_ASSUME(x) __builtin_assume(x)
	#elif defined(_MSC_VER)
		#define UE_ASSUME(x) __assume(x)
	#else
		#define UE_ASSUME(x)
	#endif
#endif

/** Improves the debugging of functions which act as pure reference casts, eliding the call and doing a direct cast from the argument to the return type */
#ifndef UE_INTRINSIC_CAST
	#define UE_INTRINSIC_CAST
#endif

/** Branch prediction hints */
#ifndef LIKELY						/* Hints compiler that expression is likely to be true, much softer than UE_ASSUME - allows (penalized by worse performance) expression to be false */
	#if ( defined(__clang__) || defined(__GNUC__) ) 
		#define LIKELY(x)			__builtin_expect(!!(x), 1)
	#else
		// the additional "!!" is added to silence "warning: equality comparison with exteraneous parenthese" messages on android
		#define LIKELY(x)			(!!(x))
	#endif
#endif

#ifndef UNLIKELY					/* Hints compiler that expression is unlikely to be true, allows (penalized by worse performance) expression to be true */
	#if ( defined(__clang__) || defined(__GNUC__) ) 
		#define UNLIKELY(x)			__builtin_expect(!!(x), 0)
	#else
		// the additional "!!" is added to silence "warning: equality comparison with exteraneous parenthese" messages on android
		#define UNLIKELY(x)			(!!(x))
	#endif
#endif

// Optimization macros (uses __pragma to enable inside a #define).
#ifndef PRAGMA_DISABLE_OPTIMIZATION_ACTUAL
	#define PRAGMA_DISABLE_OPTIMIZATION_ACTUAL
	#define PRAGMA_ENABLE_OPTIMIZATION_ACTUAL
#endif

// Disable optimization of a specific function
#ifndef DISABLE_FUNCTION_OPTIMIZATION
	#define DISABLE_FUNCTION_OPTIMIZATION
#endif

// Enable/disable optimizations for a specific function to improve build times
#define BEGIN_FUNCTION_BUILD_OPTIMIZATION UE_DISABLE_OPTIMIZATION_SHIP
#define END_FUNCTION_BUILD_OPTIMIZATION   UE_ENABLE_OPTIMIZATION_SHIP

#ifndef FORCEINLINE_DEBUGGABLE_ACTUAL
	#define FORCEINLINE_DEBUGGABLE_ACTUAL inline
#endif

#ifndef DECLARE_UINT64
	#define DECLARE_UINT64(x) x##ULL	/* Define a 64 bit immediate int **/
#endif

// Backwater of the spec. All compilers support this except microsoft, and they will soon
#ifndef TYPENAME_OUTSIDE_TEMPLATE
	#define TYPENAME_OUTSIDE_TEMPLATE	typename
#endif

// Method modifiers
#ifndef ABSTRACT
	#define ABSTRACT
#endif
#ifndef IN
	#define IN
#endif
#ifndef OUT
	#define OUT
#endif

// String constants
#ifndef LINE_TERMINATOR
	#define LINE_TERMINATOR TEXT("\n")
#endif
#ifndef LINE_TERMINATOR_ANSI
	#define LINE_TERMINATOR_ANSI "\n"
#endif

// Alignment.
#ifndef GCC_PACK
	#define GCC_PACK(n)
#endif
#ifndef GCC_ALIGN
	#define GCC_ALIGN(n)
#endif
#ifndef MS_ALIGN
	#define MS_ALIGN(n)
#endif

// MSVC pragmas - used so other platforms can remove them easily (by not defining this)
#ifndef MSVC_PRAGMA
	#define MSVC_PRAGMA(...)
#endif


// Inlining
#ifndef PRAGMA_DISABLE_INLINING
	#define PRAGMA_DISABLE_INLINING
	#define PRAGMA_ENABLE_INLINING
#endif

// Cache control
#ifndef FLUSH_CACHE_LINE
	#define FLUSH_CACHE_LINE(x)
#endif

// Prefetch
#ifndef PLATFORM_CACHE_LINE_SIZE
	#define PLATFORM_CACHE_LINE_SIZE	64
#endif

// Disable UBSan on a given function
#ifndef UE_DISABLE_UBSAN
	#define UE_DISABLE_UBSAN
#endif

// Compile-time warnings and errors. Use these as "#pragma COMPILE_WARNING("XYZ")". GCC does not expand macro parameters to _Pragma, so we can't wrap the #pragma part.
#ifdef _MSC_VER
	#define MSC_FORMAT_DIAGNOSTIC_HELPER_2(x) #x
	#define MSC_FORMAT_DIAGNOSTIC_HELPER(x) MSC_FORMAT_DIAGNOSTIC_HELPER_2(x)
	#define COMPILE_ERROR(x) __pragma(message(__FILE__ "(" MSC_FORMAT_DIAGNOSTIC_HELPER(__LINE__) "): error: " x))
	#if UE_WARNINGS_AS_ERRORS
		#define COMPILE_WARNING(x) COMPILE_ERROR(x)
	#else
		#define COMPILE_WARNING(x) __pragma(message(__FILE__ "(" MSC_FORMAT_DIAGNOSTIC_HELPER(__LINE__) "): warning: " x))
	#endif
#else
	#define GCC_DIAGNOSTIC_HELPER(x) _Pragma(#x)
	#define COMPILE_ERROR(x) GCC_DIAGNOSTIC_HELPER(GCC error x)
	#if UE_WARNINGS_AS_ERRORS
		#define COMPILE_WARNING(x) COMPILE_ERROR(x)
	#else
		#define COMPILE_WARNING(x) GCC_DIAGNOSTIC_HELPER(GCC warning x)
	#endif
#endif

// Tells the compiler to put the decorated function in a certain section (aka. segment) of the executable.
#ifndef PLATFORM_CODE_SECTION
	#define PLATFORM_CODE_SECTION(Name)
#endif

// Mark a function or type as unlikely to be used so the compiler can store it away from the warm code paths & optimize it for size rather than speed.
// Examples:
//	void UE_COLD HandleError();
//	auto ErrorHandler = []() UE_COLD { abort(); };
#if !defined(UE_COLD)
	#if defined(_MSC_VER)
		#define UE_COLD __declspec(noinline)
	#elif defined(__GNUC__) || defined(__clang__) || defined(__llvm__)
		#define UE_COLD __attribute__((cold))
	#else
		#define UE_COLD
	#endif
#endif

// These have to be forced inline on some OSes so the dynamic loader will not
// resolve to our allocators for the system libraries.
#ifndef OPERATOR_NEW_INLINE
	#define OPERATOR_NEW_INLINE FORCEINLINE
#endif

#ifndef OPERATOR_NEW_THROW_SPEC
	#define OPERATOR_NEW_THROW_SPEC
#endif
#ifndef OPERATOR_DELETE_THROW_SPEC
	#define OPERATOR_DELETE_THROW_SPEC
#endif
#ifndef OPERATOR_NEW_NOTHROW_SPEC
	#define OPERATOR_NEW_NOTHROW_SPEC throw()
#endif
#ifndef OPERATOR_DELETE_NOTHROW_SPEC
	#define OPERATOR_DELETE_NOTHROW_SPEC throw()
#endif

#ifndef checkAtCompileTime
	#define checkAtCompileTime(expr, msg) \
		EMIT_DEPRECATED_WARNING_MESSAGE("checkAtCompileTime is deprecated. Please use static_assert instead.") \
		static_assert(expr, #msg)
#endif

// DLL export and import definitions
#ifndef DLLEXPORT
	#define DLLEXPORT
	#define DLLIMPORT
#endif

// embedded app is not default (embedding UE4 in a native view, right now just for IOS and Android)
#ifndef BUILD_EMBEDDED_APP
	#define BUILD_EMBEDDED_APP  0
#endif
#ifndef FAST_BOOT_HACKS
	#define FAST_BOOT_HACKS  0
#endif

// Console ANSICHAR/TCHAR command line handling
#if PLATFORM_COMPILER_HAS_TCHAR_WMAIN
#define INT32_MAIN_INT32_ARGC_TCHAR_ARGV() int32 wmain(int32 ArgC, TCHAR* ArgV[])
#else
#define INT32_MAIN_INT32_ARGC_TCHAR_ARGV() \
int32 tchar_main(int32 ArgC, TCHAR* ArgV[]); \
int32 main(int32 ArgC, ANSICHAR* Utf8ArgV[]) \
{ \
	TCHAR** ArgV = new TCHAR*[ArgC]; \
	for (int32 a = 0; a < ArgC; a++) \
	{ \
		FUTF8ToTCHAR ConvertFromUtf8(Utf8ArgV[a]); \
		ArgV[a] = new TCHAR[ConvertFromUtf8.Length() + 1]; \
		FCString::Strncpy(ArgV[a], ConvertFromUtf8.Get(), ConvertFromUtf8.Length() + 1); \
	} \
	int32 Result = tchar_main(ArgC, ArgV); \
	for (int32 a = 0; a < ArgC; a++) \
	{ \
		delete[] ArgV[a]; \
	} \
	delete[] ArgV; \
	return Result; \
} \
int32 tchar_main(int32 ArgC, TCHAR* ArgV[])
#endif

// perform an unaligned write to almost-zero (writing to 0 will throw warnings in some compilers, plus it might actually not crash)
// this allow some platforms to change how they force a crash, in case this isn't enough
// a platform can likely just #define UE_FORCE_CRASH_AT_OFFSET to override the crash behavior
#ifndef UE_FORCE_CRASH_AT_OFFSET
	#define UE_FORCE_CRASH_AT_OFFSET(x) *(int32 *)x = 123
#endif

#ifndef UE_FORCE_CRASH
	#define UE_FORCE_CRASH() UE_FORCE_CRASH_AT_OFFSET(3)
#endif

#ifndef UE_WORKAROUND_CLANG_SVE_LTO_BUGS
#	define UE_WORKAROUND_CLANG_SVE_LTO_BUGS 0
#endif

// workaround for LTO generating invalid SVE code that crashes clang
#if UE_WORKAROUND_CLANG_SVE_LTO_BUGS
#	define UE_NO_PROFILE_SVE_BUG UE_NO_PROFILE_ATTRIBUTE
#else
#	define UE_NO_PROFILE_SVE_BUG
#endif

// By default we assume that platforms require us to not have `nullptr` as arguments to
// memory operations (like `memcpy`, `memset`, `memmove`, etc). `n3322` will define that
// having `nullptr` as arguments to these functions with a size of 0 is ok, rather than
// the UB it caused previously.
#ifndef UE_PLATFORM_REQUIRES_NONNULL_MEMOP_ARGS
#	define UE_PLATFORM_REQUIRES_NONNULL_MEMOP_ARGS 1
#endif

//--------------------------------------------------------------------------------------------
// POD types refactor for porting old code:
//
// Replace 'FLOAT' with 'float'
// Replace 'DOUBLE' with 'double'
// Replace 'INT' with 'int32' (make sure not to replace INT text in localization strings!)
// Replace 'UINT' with 'uint32'
// Replace 'DWORD' with 'uint32' (DWORD is still used in Windows platform source files)
// Replace 'WORD' with 'uint16'
// Replace 'USHORT' with 'uint16'
// Replace 'SWORD' with 'int16'
// Replace 'QWORD' with 'uint64'
// Replace 'SQWORD' with 'int64'
// Replace 'BYTE' with 'uint8'
// Replace 'SBYTE' with 'int8'
// Replace 'BITFIELD' with 'uint32'
// Replace 'UBOOL' with 'bool'
// Replace 'FALSE' with 'false' and 'TRUE' with true.
// Make sure any platform API uses its own bool types.
// For example WinAPI uses BOOL, FALSE and TRUE otherwise sometimes it doesn't work properly
// or it doesn't compile.
// Replace 'ExtractUBOOLFromBitfield' with 'ExtractBoolFromBitfield'
// Replace 'SetUBOOLInBitField' with 'SetBoolInBitField'
// Replace 'ParseUBOOL' with 'FParse::Bool'
// Replace 'MINBYTE', with 'MIN_uint8'
// Replace 'MINWORD', with 'MIN_uint16'
// Replace 'MINDWORD', with 'MIN_uint32'
// Replace 'MINSBYTE', with 'MIN_int8'
// Replace 'MINSWORD', with 'MIN_int16'
// Replace 'MAXINT', with 'MAX_int32'
// Replace 'MAXBYTE', with 'MAX_uint8'
// Replace 'MAXWORD', with 'MAX_uint16'
// Replace 'MAXDWORD', with 'MAX_uint32'
// Replace 'MAXSBYTE', with 'MAX_int8'
// Replace 'MAXSWORD', with 'MAX_int16'
// Replace 'MAXINT', with 'MAX_int32'
//--------------------------------------------------------------------------------------------

//------------------------------------------------------------------
// Transfer the platform types to global types
//------------------------------------------------------------------

//~ Unsigned base types.
/// An 8-bit unsigned integer.
typedef FPlatformTypes::uint8		uint8;
/// A 16-bit unsigned integer.
typedef FPlatformTypes::uint16		uint16;
/// A 32-bit unsigned integer.
typedef FPlatformTypes::uint32		uint32;
/// A 64-bit unsigned integer.
typedef FPlatformTypes::uint64		uint64;

//~ Signed base types.
/// An 8-bit signed integer.
typedef	FPlatformTypes::int8		int8;
/// A 16-bit signed integer.
typedef FPlatformTypes::int16		int16;
/// A 32-bit signed integer.
typedef FPlatformTypes::int32		int32;
/// A 64-bit signed integer.
typedef FPlatformTypes::int64		int64;

//~ Character types.
/// An ANSI character. Normally a signed type.
typedef FPlatformTypes::ANSICHAR	ANSICHAR;
/// A wide character. Normally a signed type.
typedef FPlatformTypes::WIDECHAR	WIDECHAR;
/// Either ANSICHAR or WIDECHAR, depending on whether the platform supports wide characters or the requirements of the licensee.
typedef FPlatformTypes::TCHAR		TCHAR;
/// An 8-bit character containing a UTF8 (Unicode, 8-bit, variable-width) code unit.
typedef FPlatformTypes::UTF8CHAR	UTF8CHAR;
/// A 16-bit character containing a UCS2 (Unicode, 16-bit, fixed-width) code unit, used for compatibility with 'Windows TCHAR' across multiple platforms.
typedef FPlatformTypes::CHAR16		UCS2CHAR;
/// A 16-bit character containing a UTF16 (Unicode, 16-bit, variable-width) code unit.
typedef FPlatformTypes::CHAR16		UTF16CHAR;
/// A 32-bit character containing a UTF32 (Unicode, 32-bit, fixed-width) code unit.
typedef FPlatformTypes::UTF32CHAR	UTF32CHAR;

/// An unsigned integer the same size as a pointer
typedef FPlatformTypes::UPTRINT UPTRINT;
/// A signed integer the same size as a pointer
typedef FPlatformTypes::PTRINT PTRINT;
/// An unsigned integer the same size as a pointer, the same as UPTRINT
typedef FPlatformTypes::SIZE_T SIZE_T;
/// An integer the same size as a pointer, the same as PTRINT
typedef FPlatformTypes::SSIZE_T SSIZE_T;

/// The type of the NULL constant.
typedef FPlatformTypes::TYPE_OF_NULL	TYPE_OF_NULL;
/// The type of the C++ nullptr keyword.
typedef FPlatformTypes::TYPE_OF_NULLPTR	TYPE_OF_NULLPTR;

//------------------------------------------------------------------
// Test the global types
//------------------------------------------------------------------
namespace TypeTests
{
	template <typename A, typename B>
	inline constexpr bool TAreTypesEqual_V = false;

	template <typename T>
	inline constexpr bool TAreTypesEqual_V<T, T> = true;

#if PLATFORM_TCHAR_IS_UTF8CHAR
	static_assert(sizeof(TCHAR) == 1, "TCHAR size must be 1 byte.");
#else
	static_assert(sizeof(TCHAR) == 2, "TCHAR size must be 2 bytes.");
#endif

	static_assert(!PLATFORM_WCHAR_IS_4_BYTES || sizeof(wchar_t) == 4, "wchar_t size must be 4 bytes.");
	static_assert(PLATFORM_WCHAR_IS_4_BYTES || sizeof(wchar_t) == 2, "wchar_t size must be 2 bytes.");

	static_assert(sizeof(void*) == 8, "Pointer size must be 64-bit.");

	static_assert(char(-1) < char(0), "Unsigned char type test failed.");

	static_assert((!TAreTypesEqual_V<ANSICHAR, WIDECHAR>),  "ANSICHAR and WIDECHAR should be different types.");
	static_assert((!TAreTypesEqual_V<ANSICHAR, UTF8CHAR>),  "ANSICHAR and UTF8CHAR should be different types.");
#if !PLATFORM_UCS2CHAR_IS_UTF16CHAR
	// We want these types to be different, because we want to be able to determine whether an encoding
	// is fixed-width (UCS2CHAR) or variable-width (UTF16CHAR) at compile time.
	static_assert((!TAreTypesEqual_V<UCS2CHAR, UTF16CHAR>), "UCS2CHAR and UTF16CHAR should be different types.");
#else
	// We don't want these types to be equal, but while this macro exists, they ought to be equal
	static_assert(TAreTypesEqual_V<UCS2CHAR, UTF16CHAR>, "UCS2CHAR and UTF16CHAR are expected to be the same type.");
#endif
	static_assert((!TAreTypesEqual_V<ANSICHAR, UCS2CHAR>),  "ANSICHAR and UCS2CHAR should be different types.");
	static_assert((!TAreTypesEqual_V<WIDECHAR, UCS2CHAR>),  "WIDECHAR and UCS2CHAR should be different types.");
	static_assert(TAreTypesEqual_V<TCHAR, ANSICHAR> || TAreTypesEqual_V<TCHAR, WIDECHAR> || TAreTypesEqual_V<TCHAR, UTF8CHAR>, "TCHAR should either be ANSICHAR, WIDECHAR or UTF8CHAR.");

#if PLATFORM_WIDECHAR_IS_CHAR16
	static_assert(TAreTypesEqual_V<WIDECHAR, char16_t>, "WIDECHAR should be char16_t");
#else
	static_assert(TAreTypesEqual_V<WIDECHAR, wchar_t>, "WIDECHAR should be wchar_t");
#endif

	static_assert(sizeof(uint8) == 1, "uint8 type size test failed.");
	static_assert(int32(uint8(-1)) == 0xFF, "uint8 type sign test failed.");

	static_assert(sizeof(uint16) == 2, "uint16 type size test failed.");
	static_assert(int32(uint16(-1)) == 0xFFFF, "uint16 type sign test failed.");

	static_assert(sizeof(uint32) == 4, "uint32 type size test failed.");
	static_assert(int64(uint32(-1)) == int64(0xFFFFFFFF), "uint32 type sign test failed.");

	static_assert(sizeof(uint64) == 8, "uint64 type size test failed.");
	static_assert(uint64(-1) > uint64(0), "uint64 type sign test failed.");


	static_assert(sizeof(int8) == 1, "int8 type size test failed.");
	static_assert(int32(int8(-1)) == -1, "int8 type sign test failed.");

	static_assert(sizeof(int16) == 2, "int16 type size test failed.");
	static_assert(int32(int16(-1)) == -1, "int16 type sign test failed.");

	static_assert(sizeof(int32) == 4, "int32 type size test failed.");
	static_assert(int64(int32(-1)) == int64(-1), "int32 type sign test failed.");

	static_assert(sizeof(int64) == 8, "int64 type size test failed.");
	static_assert(int64(-1) < int64(0), "int64 type sign test failed.");

	static_assert(sizeof(ANSICHAR) == 1, "ANSICHAR type size test failed.");
	static_assert(int32(ANSICHAR(-1)) == -1, "ANSICHAR type sign test failed.");

	static_assert(sizeof(WIDECHAR) == 2 || sizeof(WIDECHAR) == 4, "WIDECHAR type size test failed.");

	static_assert(sizeof(UTF8CHAR)  == 1, "UTF8CHAR type size test failed.");
	static_assert(sizeof(UCS2CHAR)  == 2, "UCS2CHAR type size test failed.");
	static_assert(sizeof(UTF16CHAR) == 2, "UTF16CHAR type size test failed.");
	static_assert(sizeof(UTF32CHAR) == 4, "UTF32CHAR type size test failed.");

	static_assert(sizeof(PTRINT) == sizeof(void *), "PTRINT type size test failed.");
	static_assert(PTRINT(-1) < PTRINT(0), "PTRINT type sign test failed.");

	static_assert(sizeof(UPTRINT) == sizeof(void *), "UPTRINT type size test failed.");
	static_assert(UPTRINT(-1) > UPTRINT(0), "UPTRINT type sign test failed.");

	static_assert(sizeof(SIZE_T) == sizeof(void *), "SIZE_T type size test failed.");
	static_assert(SIZE_T(-1) > SIZE_T(0), "SIZE_T type sign test failed.");
}

// Platform specific compiler setup.
#include COMPILED_PLATFORM_HEADER(PlatformCompilerSetup.h)


#define UTF8TEXT_PASTE(x)  u8 ## x
#define UTF16TEXT_PASTE(x) u ## x
#if PLATFORM_WIDECHAR_IS_CHAR16
	#define WIDETEXT_PASTE(x)  UTF16TEXT_PASTE(x)
#else
	#define WIDETEXT_PASTE(x)  L ## x
#endif

// If we don't have a platform-specific define for the TEXT macro, define it now.
#if !defined(TEXT) && !UE_BUILD_DOCS
	#if PLATFORM_TCHAR_IS_UTF8CHAR
		#define TEXT_PASTE(x) UTF8TEXT(x)
	#else
		#define TEXT_PASTE(x) WIDETEXT(x)
	#endif
	#define TEXT(x) TEXT_PASTE(x)
#endif

#define UE_UTF8TEXT_IS_CONSTEXPR 1 UE_DEPRECATED_MACRO(5.7, "UE_UTF8TEXT_IS_CONSTEXPR has been deprecated.")

namespace UE::Core::Private
{
	template <typename CharType>
	void CharTextStaticAssert()
	{
		static_assert(sizeof(CharType) == 0, "Unsupported char type passed to CHARTEXT");
	}
}

#define UTF8TEXT(x) UTF8TEXT_PASTE(x)
#define UTF16TEXT(x) UTF16TEXT_PASTE(x)
#define WIDETEXT(str) WIDETEXT_PASTE(str)

// Expands out to x, TEXT(x), WIDETEXT(x) or UTF8TEXT(x) depending on CharType
#define CHARTEXT(CharType, x) \
	( \
		[]() -> decltype(auto) \
		{ \
			/* We expect <type_traits> to already have been included any time the CHARTEXT macro is used */ \
			using UnqualifiedCharType = std::remove_cv_t<CharType>; \
			if constexpr (std::is_same_v<UnqualifiedCharType, ANSICHAR>) \
			{ \
				return x; \
			} \
			else if constexpr (std::is_same_v<UnqualifiedCharType, TCHAR>) \
			{ \
				return TEXT(x); \
			} \
			else if constexpr (std::is_same_v<UnqualifiedCharType, WIDECHAR>) \
			{ \
				return WIDETEXT(x); \
			} \
			else if constexpr (std::is_same_v<UnqualifiedCharType, UTF8CHAR>) \
			{ \
				return UTF8TEXT(x); \
			} \
			else \
			{ \
				/* We want a compile error, so forward to a templated function because of the static_assert(false) problem */ \
				UE::Core::Private::CharTextStaticAssert<CharType>(); \
				return x; \
			} \
		}() \
	)

// IWYU pragma: end_exports

