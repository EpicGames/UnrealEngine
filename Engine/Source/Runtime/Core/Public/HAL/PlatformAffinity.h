// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include COMPILED_PLATFORM_HEADER(PlatformAffinity.h) // IWYU pragma: export

class FPlatformAffinity : public FPlatformAffinityBase
{
public:
	/*
	* Sets the new affinity mask used for the audio render threads
	* @note It only takes effect for new audio render threads
	*/
	static CORE_API void SetAudioRenderThreadMask(const uint64 InNewValue);

	/*
	* Returns the affinity mask used for the audio render threads
	*/
	static const CORE_API uint64 GetAudioRenderThreadMask();
};

struct FThreadAffinity
{
	uint64 ThreadAffinityMask = FPlatformAffinity::GetNoAffinityMask();
	uint16 ProcessorGroup = 0;
};
