// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/ThreadSafeCounter.h"
#include "Misc/ScopeLock.h"
#include "HAL/DiskUtilizationTracker.h"
#include "HAL/ThreadSafeCounter.h"
#include "Misc/ScopeLock.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformProcess.h"

#define MANAGE_FILE_HANDLES (#) // this is not longer used, this will error on any attempt to use it

class FRegisteredFileHandle : public IFileHandle
{
private:
	friend class FFileHandleRegistry;

	FRegisteredFileHandle* NextLink = nullptr;
	FRegisteredFileHandle* PreviousLink = nullptr;
	uint32 ReadRequestCount = 0;
	bool bIsOpenAndAvailableForClosing = false;
};


class FFileHandleRegistry
{
public:
	FFileHandleRegistry(int32 InMaxOpenHandles)
		: MaxOpenHandles(InMaxOpenHandles)
	{}

	virtual ~FFileHandleRegistry() = default;

	[[nodiscard]] FRegisteredFileHandle* InitialOpenFile(const TCHAR* Filename)
	{
		if (HandlesCurrentlyInUse.Increment() > MaxOpenHandles)
		{
			FreeHandles();
		}

		FRegisteredFileHandle* Handle = PlatformInitialOpenFile(Filename);
		if (Handle != nullptr)
		{
			FScopeLock Lock(&LockSection);
			LinkToTail(Handle);
		}
		else
		{
			HandlesCurrentlyInUse.Decrement();
		}

		return Handle;
	}

	void UnTrackAndCloseFile(FRegisteredFileHandle* Handle)
	{
		bool bWasOpen = false;
		{
			FScopeLock Lock(&LockSection);
			check(Handle->ReadRequestCount == 0);
			if (Handle->bIsOpenAndAvailableForClosing)
			{
				UnLink(Handle);
				bWasOpen = true;
			}
		}
		if (bWasOpen)
		{
			PlatformCloseFile(Handle);
			HandlesCurrentlyInUse.Decrement();
		}
	}

	// Only returns false if the platform handle cannot be reopened.
	// TrackEndRead should only be called if TrackStartRead succeeded
	[[nodiscard]] bool TrackStartRead(FRegisteredFileHandle* Handle)
	{
		{
			FScopeLock Lock(&LockSection);
			
			if (Handle->ReadRequestCount++ != 0)
			{
				return true;
			}

			if (Handle->bIsOpenAndAvailableForClosing)
			{
				UnLink(Handle);
				return true;
			}
		}

		if (HandlesCurrentlyInUse.Increment() > MaxOpenHandles)
		{
			FreeHandles();
		}

		// can do this out of the lock, in case it's slow
		bool bSuccess = PlatformReopenFile(Handle);

		if (!bSuccess)
		{
			FScopeLock Lock(&LockSection);
			--Handle->ReadRequestCount;
		}

		return bSuccess;
	}

	// TrackEndRead should only be called if TrackStartRead succeeded
	void TrackEndRead(FRegisteredFileHandle* Handle)
	{
		FScopeLock Lock(&LockSection);

		if (--Handle->ReadRequestCount == 0)
		{
			LinkToTail(Handle);
		}
	}

protected:
	virtual FRegisteredFileHandle* PlatformInitialOpenFile(const TCHAR* Filename) = 0;
	virtual bool PlatformReopenFile(FRegisteredFileHandle*) = 0;
	virtual void PlatformCloseFile(FRegisteredFileHandle*) = 0;

private:

	void FreeHandles()
	{
		// do we need to make room for a file handle?
		while (HandlesCurrentlyInUse.GetValue() > MaxOpenHandles)
		{
			FRegisteredFileHandle* ToBeClosed = nullptr;
			{
				FScopeLock Lock(&LockSection);
				ToBeClosed = PopFromHead();
			}
			if (ToBeClosed)
			{
				// close it, freeing up space for new file to open
				PlatformCloseFile(ToBeClosed);
				HandlesCurrentlyInUse.Decrement();
			}
			else
			{
				FPlatformMisc::LowLevelOutputDebugString(TEXT("FFileHandleRegistry: Spinning because we are actively reading from more file handles than we have possible handles.\r\n"));
				FPlatformProcess::SleepNoStats(.1f);
			}
		}
	}

	void LinkToTail(FRegisteredFileHandle* Handle)
	{
		check(!Handle->PreviousLink && !Handle->NextLink && !Handle->bIsOpenAndAvailableForClosing);
		Handle->bIsOpenAndAvailableForClosing = true;
		if (OpenAndAvailableForClosingTail)
		{
			Handle->PreviousLink = OpenAndAvailableForClosingTail;
			check(!OpenAndAvailableForClosingTail->NextLink);
			OpenAndAvailableForClosingTail->NextLink = Handle;
		}
		else
		{
			check(!OpenAndAvailableForClosingHead);
			OpenAndAvailableForClosingHead = Handle;
		}
		OpenAndAvailableForClosingTail = Handle;
	}

	void UnLink(FRegisteredFileHandle* Handle)
	{
		if (OpenAndAvailableForClosingHead == Handle)
		{
			verify(PopFromHead() == Handle);
			return;
		}
		check(Handle->bIsOpenAndAvailableForClosing);
		Handle->bIsOpenAndAvailableForClosing = false;
		if (OpenAndAvailableForClosingTail == Handle)
		{
			check(OpenAndAvailableForClosingHead && OpenAndAvailableForClosingHead != Handle && Handle->PreviousLink);
			OpenAndAvailableForClosingTail = Handle->PreviousLink;
			OpenAndAvailableForClosingTail->NextLink = nullptr;
			Handle->NextLink = nullptr;
			Handle->PreviousLink = nullptr;
			return;
		}
		check(Handle->NextLink && Handle->PreviousLink);
		Handle->NextLink->PreviousLink = Handle->PreviousLink;
		Handle->PreviousLink->NextLink = Handle->NextLink;
		Handle->NextLink = nullptr;
		Handle->PreviousLink = nullptr;

	}

	FRegisteredFileHandle* PopFromHead()
	{
		FRegisteredFileHandle* Result = OpenAndAvailableForClosingHead;
		if (Result)
		{
			check(!Result->PreviousLink);
			check(Result->bIsOpenAndAvailableForClosing);
			Result->bIsOpenAndAvailableForClosing = false;
			OpenAndAvailableForClosingHead = Result->NextLink;
			if (!OpenAndAvailableForClosingHead)
			{
				check(OpenAndAvailableForClosingTail == Result);
				OpenAndAvailableForClosingTail = nullptr;
			}
			else
			{
				check(OpenAndAvailableForClosingHead->PreviousLink == Result);
				OpenAndAvailableForClosingHead->PreviousLink = nullptr;
			}
			Result->NextLink = nullptr;
			Result->PreviousLink = nullptr;
		}
		return Result;
	}

	// critical section to protect the below arrays
	FCriticalSection LockSection;

	int32 MaxOpenHandles = 0;

	FRegisteredFileHandle* OpenAndAvailableForClosingHead = nullptr;
	FRegisteredFileHandle* OpenAndAvailableForClosingTail = nullptr;

	FThreadSafeCounter HandlesCurrentlyInUse;
};

class FFileHandleRegistryReadTracker
{
public:
	FFileHandleRegistryReadTracker(FFileHandleRegistry& InFileRegistry, FRegisteredFileHandle& InHandle, bool bTrackRead = true)
		: FileRegistry(InFileRegistry)
		, Handle(InHandle)
	{
		if (bTrackRead)
		{
			State = FileRegistry.TrackStartRead(&Handle)	? ETrackingState::Success
															: ETrackingState::Failure;
		}
		else
		{
			State = ETrackingState::Skip;
		}
	}

	~FFileHandleRegistryReadTracker()
	{
		if (State == ETrackingState::Success)
		{
			FileRegistry.TrackEndRead(&Handle);
		}
	}

	bool IsValid() const
	{
		return State != ETrackingState::Failure;
	}

private:
	FFileHandleRegistry& FileRegistry;
	FRegisteredFileHandle& Handle;

	enum class ETrackingState : uint8
	{
		Failure,
		Skip,
		Success
	} State;
};
