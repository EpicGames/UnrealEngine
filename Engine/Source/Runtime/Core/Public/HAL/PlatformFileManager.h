// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "CoreTypes.h"
#include "Templates/Atomic.h"

#ifndef USE_ATOMIC_PLATFORM_FILE
	#define USE_ATOMIC_PLATFORM_FILE (WITH_EDITOR)
#endif

/**
* Platform File chain manager.
**/
class FPlatformFileManager
{
#if USE_ATOMIC_PLATFORM_FILE
	/** Currently used platform file. */
	TAtomic<class IPlatformFile*> TopmostPlatformFile;
	TAtomic<class IPlatformFile*> PhysicalPlatformFile;
#else
	/** Currently used platform file. */
	class IPlatformFile* TopmostPlatformFile;
	class IPlatformFile* PhysicalPlatformFile;
#endif

public:

	/** Constructor. */
	CORE_API FPlatformFileManager( );

	/**
	 * Gets the currently used platform file.
	 *
	 * @return Reference to the currently used platform file.
	 */
	CORE_API IPlatformFile& GetPlatformFile( );

	CORE_API IPlatformFile& GetPlatformPhysical();

	/**
	 * Sets the current platform file.
	 *
	 * @param NewTopmostPlatformFile Platform file to be used.
	 */
	CORE_API void SetPlatformFile( IPlatformFile& NewTopmostPlatformFile );

	CORE_API void SetPlatformPhysical(IPlatformFile& NewPhysicalPlatformFile);

	/**
	 * Finds a platform file in the chain of active platform files.
	 *
	 * @param Name of the platform file.
	 * @return Pointer to the active platform file or nullptr if the platform file was not found.
	 */
	CORE_API IPlatformFile* FindPlatformFile( const TCHAR* Name );

	/**
	 * Creates a new platform file instance.
	 *
	 * @param Name of the platform file to create.
	 * @return Platform file instance of the platform file type was found, nullptr otherwise.
	 */
	CORE_API IPlatformFile* GetPlatformFile( const TCHAR* Name );

	/**
	 * calls Tick on the platform files in the TopmostPlatformFile chain
	 */
	CORE_API void TickActivePlatformFile();

	/**
	* Performs additional initialization when the new async IO is enabled.
	*/
	CORE_API void InitializeNewAsyncIO();

	/**
	 * Gets FPlatformFileManager Singleton.
	 */
	static CORE_API FPlatformFileManager& Get( );

	/**
	* Removes the specified file wrapper from the platform file wrapper chain.
	*
	* THIS IS EXTREMELY DANGEROUS AFTER THE ENGINE HAS BEEN INITIALIZED AS WE MAY BE MODIFYING
	* THE WRAPPER CHAIN WHILE THINGS ARE BEING LOADED
	*
	* @param The platform file to remove.
	*/
	CORE_API void RemovePlatformFile(IPlatformFile* PlatformFileToRemove);

	/**
	* Inserts a new platform file into the platform file wrapper chain.
	* The file is inserted before NewPlatformFile->GetLowerLevel().
	*
	* THIS IS EXTREMELY DANGEROUS AFTER THE ENGINE HAS BEEN INITIALIZED AS WE MAY BE MODIFYING
	* THE WRAPPER CHAIN WHILE THINGS ARE BEING LOADED
	*
	* @param The platform file to insert.
	* @return true if the platform file was inserted.
	*/
	CORE_API bool InsertPlatformFile(IPlatformFile* NewPlatformFile);
};
