// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

// All the different platform-specific includes
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformString.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/PlatformMath.h"
#include "HAL/PlatformNamedPipe.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PlatformTLS.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformAffinity.h"
