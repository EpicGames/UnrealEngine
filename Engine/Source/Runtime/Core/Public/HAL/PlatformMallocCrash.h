// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"

#include "GenericPlatform/GenericPlatformMallocCrash.h"

#if PLATFORM_USES_STACKBASED_MALLOC_CRASH
typedef FGenericStackBasedMallocCrash FPlatformMallocCrash;
#else
typedef FGenericPlatformMallocCrash FPlatformMallocCrash;
#endif
