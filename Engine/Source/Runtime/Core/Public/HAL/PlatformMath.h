// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformMath.h"

#include COMPILED_PLATFORM_HEADER(PlatformMath.h)
