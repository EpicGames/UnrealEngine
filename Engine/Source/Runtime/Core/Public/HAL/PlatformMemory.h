// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformMemory.h"

#include COMPILED_PLATFORM_HEADER(PlatformMemory.h) // IWYU pragma: export
