// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/PlatformMemory.h"

namespace PlatformMemoryHelpers
{

	//@return platform specific current memory statistics - caches value for same frame to avoid repeated costly platform calls.
	CORE_API FPlatformMemoryStats GetFrameMemoryStats();
}