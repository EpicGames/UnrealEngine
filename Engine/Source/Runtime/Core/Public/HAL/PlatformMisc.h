// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include COMPILED_PLATFORM_HEADER(PlatformMisc.h)
#include "ProfilingDebugging/CpuProfilerTrace.h"

#ifndef UE_DEBUG_BREAK
#error UE_DEBUG_BREAK is not defined for this platform
#endif

#ifndef PLATFORM_USES_ANSI_STRING_FOR_EXTERNAL_PROFILING
#error PLATFORM_USES_ANSI_STRING_FOR_EXTERNAL_PROFILING is not defined.
#endif

#ifndef PLATFORM_LIMIT_PROFILER_UNIQUE_NAMED_EVENTS
#if defined(FRAMEPRO_ENABLED) && FRAMEPRO_ENABLED
 // If framepro is enabled, we need to limit the number of unique events
 // This define prevents us emitting "Frame N" events, etc
 #define PLATFORM_LIMIT_PROFILER_UNIQUE_NAMED_EVENTS 1
#else
 #define PLATFORM_LIMIT_PROFILER_UNIQUE_NAMED_EVENTS 0
#endif
#endif

#ifndef PLATFORM_EMPTY_BASES
#define PLATFORM_EMPTY_BASES
#endif

#ifndef ALLOW_NAMED_EVENTS_IN_TEST
#define ALLOW_NAMED_EVENTS_IN_TEST 1	// enabled in Test by default to facilitate auto perf testing
#endif

// Master switch for scoped named events
#define ENABLE_NAMED_EVENTS (!UE_BUILD_SHIPPING && (!UE_BUILD_TEST || ALLOW_NAMED_EVENTS_IN_TEST))

#if PLATFORM_USES_ANSI_STRING_FOR_EXTERNAL_PROFILING
	#define NAMED_EVENT_STR(x) x
#else
	#define NAMED_EVENT_STR(x) TEXT(x)
#endif

#if ENABLE_NAMED_EVENTS

class FScopedNamedEventConditional
{
public:
	FScopedNamedEventConditional(const struct FColor& Color, const TCHAR* Text, bool bCondition)
		: bStarted(bCondition)
	{
		if (bCondition)
		{
			FPlatformMisc::BeginNamedEvent(Color, Text);
		}
	}

	FScopedNamedEventConditional(const struct FColor& Color, const ANSICHAR* Text, bool bCondition)
		: bStarted(bCondition)
	{
		if (bCondition)
		{
			FPlatformMisc::BeginNamedEvent(Color, Text);
		}
	}

	~FScopedNamedEventConditional()
	{
		if (bStarted)
		{
			FPlatformMisc::EndNamedEvent();
		}
	}

private:
	bool bStarted;
};

class FScopedNamedEvent
{
public:

	FScopedNamedEvent(const struct FColor& Color, const TCHAR* Text)
	{
		FPlatformMisc::BeginNamedEvent(Color, Text);
	}

	FScopedNamedEvent(const struct FColor& Color, const ANSICHAR* Text)
	{
		FPlatformMisc::BeginNamedEvent(Color, Text);
	}

	~FScopedNamedEvent()
	{
		FPlatformMisc::EndNamedEvent();
	}	
};

//
// Scoped named event class for constant (compile-time) strings literals.
//
// BeginNamedEventStatic works the same as BeginNamedEvent, but should only be passed a compile-time string literal.
// Some platform profilers can optimize the case where strings for certain events are constant.
//
class FScopedNamedEventStatic
{
public:

	FScopedNamedEventStatic(const struct FColor& Color, const TCHAR* Text)
	{
#if PLATFORM_IMPLEMENTS_BeginNamedEventStatic
		FPlatformMisc::BeginNamedEventStatic(Color, Text);
#else
		FPlatformMisc::BeginNamedEvent(Color, Text);
#endif
	}

	FScopedNamedEventStatic(const struct FColor& Color, const ANSICHAR* Text)
	{
#if PLATFORM_IMPLEMENTS_BeginNamedEventStatic
		FPlatformMisc::BeginNamedEventStatic(Color, Text);
#else
		FPlatformMisc::BeginNamedEvent(Color, Text);
#endif
	}

	~FScopedNamedEventStatic()
	{
		FPlatformMisc::EndNamedEvent();
	}
};

// Conditional version of FScopedNamedEventStatic
class FScopedNamedEventConditionalStatic
{
public:

	FScopedNamedEventConditionalStatic(const struct FColor& Color, const TCHAR* Text, bool bCondition)
		: bStarted(bCondition)
	{
		if (bCondition)
		{
#if PLATFORM_IMPLEMENTS_BeginNamedEventStatic
			FPlatformMisc::BeginNamedEventStatic(Color, Text);
#else
			FPlatformMisc::BeginNamedEvent(Color, Text);
#endif
		}
	}

	FScopedNamedEventConditionalStatic(const struct FColor& Color, const ANSICHAR* Text, bool bCondition)
		: bStarted(bCondition)
	{
		if (bCondition)
		{
#if PLATFORM_IMPLEMENTS_BeginNamedEventStatic
			FPlatformMisc::BeginNamedEventStatic(Color, Text);
#else
			FPlatformMisc::BeginNamedEvent(Color, Text);
#endif
		}
	}

	~FScopedNamedEventConditionalStatic()
	{
		if (bStarted)
		{
			FPlatformMisc::EndNamedEvent();
		}
	}

private:
	bool bStarted;
};


// Lightweight scoped named event separate from stats system.  Will be available in test builds.  
// Events cost profiling overhead so use them judiciously in final code.

#define SCOPED_NAMED_EVENT(Name, Color)\
	FScopedNamedEventStatic ANONYMOUS_VARIABLE(NamedEvent_##Name##_)(Color, NAMED_EVENT_STR(#Name));\
	TRACE_CPUPROFILER_EVENT_SCOPE(Name);

#define SCOPED_NAMED_EVENT_FSTRING(Text, Color)\
	FScopedNamedEvent ANONYMOUS_VARIABLE(NamedEvent_)(Color, *Text);\
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*Text);

#define SCOPED_NAMED_EVENT_TCHAR(Text, Color)\
	FScopedNamedEvent ANONYMOUS_VARIABLE(NamedEvent_)(Color, Text);\
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(Text);

#define SCOPED_NAMED_EVENT_TEXT(Text, Color)\
	FScopedNamedEventStatic ANONYMOUS_VARIABLE(NamedEvent_)(Color, NAMED_EVENT_STR(Text));\
	TRACE_CPUPROFILER_EVENT_SCOPE_STR(Text);

#define SCOPED_NAMED_EVENT_F(Format, Color, ...)\
	FScopedNamedEvent ANONYMOUS_VARIABLE(NamedEvent_)(Color, *FString::Printf(Format, __VA_ARGS__));\
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(Format, __VA_ARGS__));

#define SCOPED_NAMED_EVENT_TCHAR_CONDITIONAL(Text, Color, bCondition)\
	FScopedNamedEventConditional ANONYMOUS_VARIABLE(NamedEvent_)(Color, Text, (bCondition));\
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(Text, (bCondition));

#else

#define SCOPED_NAMED_EVENT(Name, Color)									TRACE_CPUPROFILER_EVENT_SCOPE(Name);
#define SCOPED_NAMED_EVENT_FSTRING(Text, Color)							TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*Text);
#define SCOPED_NAMED_EVENT_TCHAR(Text, Color)							TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(Text);
#define SCOPED_NAMED_EVENT_TEXT(Text, Color)							TRACE_CPUPROFILER_EVENT_SCOPE_STR(Text);
#define SCOPED_NAMED_EVENT_F(Format, Color, ...)						TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(Format, __VA_ARGS__));
#define SCOPED_NAMED_EVENT_TCHAR_CONDITIONAL(Text, Color, bCondition)	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(Text, (bCondition));

#endif

// For timing OnEnterBackground tasks. This can be time sensitive on some platforms
class FScopedEnterBackgroundEvent
{
public:

	FScopedEnterBackgroundEvent(const TCHAR* Text)
	{
		FPlatformMisc::BeginEnterBackgroundEvent(Text);
	}

	~FScopedEnterBackgroundEvent()
	{
		FPlatformMisc::EndEnterBackgroundEvent();
	}
};

// Note: we don't use ANONYMOUS_VARIABLE here because we might want to view the event in a crash dump watch window
#define SCOPED_ENTER_BACKGROUND_EVENT(Name)	\
	FScopedEnterBackgroundEvent EnterBackgroundEvent_##Name##_(TEXT(#Name)); \
	QUICK_SCOPE_CYCLE_COUNTER(Name);


#ifdef PLATFORM_COMPILER_IWYU

// There are limitations to what IWYU can read out of the ast. decltype() inside unused template parameter default initializer is one example
// In those cases we need to give IWYU something it can use to know what is needed
namespace UE::Core::Private
{
	template <typename T, bool IsSameTypes = true> struct IwyuTestSize { enum { Value = 0 }; };
	template <typename T> struct IwyuTestSize<T, false> { enum { Value = sizeof(T) }; };
}
#define IWYU_MARKUP_IMPLICIT_CAST(From, To) UE::Core::Private::IwyuTestSize<From, std::is_same<typename std::remove_cv<To>::type, typename std::remove_cv<From>::type>::value>::Value

#else
#define IWYU_MARKUP_IMPLICIT_CAST(From, To)
#endif
