// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"

#if PLATFORM_WINDOWS
#include "Windows/WindowsPlatformNamedPipe.h"
#else
#endif
