// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include COMPILED_PLATFORM_HEADER(PlatformProcess.h) // IWYU pragma: export

namespace UE::HAL
{
	struct [[nodiscard]] FPipe
	{
		constexpr FPipe() = default;
		FPipe(const FPipe&) = delete;
		FPipe& operator=(const FPipe&) = delete;
		
		constexpr FPipe(std::nullptr_t)
		{
		}
		
		constexpr FPipe(FPipe&& Other)
			: ReadHandle{std::exchange(Other.ReadHandle, {})}
			, WriteHandle{std::exchange(Other.WriteHandle, {})}
		{
		}
		
		~FPipe()
		{
			FPlatformProcess::ClosePipe(ReadHandle, WriteHandle);
		}
		
		FPipe& operator=(std::nullptr_t)
		{
			FPlatformProcess::ClosePipe(std::exchange(ReadHandle, {}), std::exchange(WriteHandle, {}));
			return *this;
		}
		
		constexpr FPipe& operator=(FPipe&& Other)
		{
			Swap(ReadHandle, Other.ReadHandle);
			Swap(WriteHandle, Other.WriteHandle);
			return *this;
		}
		
		[[nodiscard]] bool operator==(std::nullptr_t) const
		{
			return !ReadHandle;
		}
		
		[[nodiscard]] explicit operator bool() const
		{
			return bool(ReadHandle);
		}

	protected:
		FReadHandle ReadHandle;
		FWriteHandle WriteHandle;
		
		FPipe(TTuple<FReadHandle, FWriteHandle> Pipe)
			: ReadHandle{Pipe.Get<FReadHandle>()}
			, WriteHandle{Pipe.Get<FWriteHandle>()}
		{
		}
		
		explicit FPipe(bool bWritePipeLocal)
			: FPipe{[&bWritePipeLocal]
			{
				void* ReadPipe;
				void* WritePipe;
				if (!FPlatformProcess::CreatePipe(ReadPipe, WritePipe, bWritePipeLocal))
				{
					ReadPipe = nullptr;
					WritePipe = nullptr;
				}
				return TTuple<FReadHandle, FWriteHandle>{FReadHandle{ReadPipe}, FWriteHandle{WritePipe}};
			}()}
		{
		}
	};
	
	inline constexpr struct FNewPipe
	{
	}
	NewPipe;
	
	struct [[nodiscard]] FInputPipe: FPipe
	{
		using FPipe::FPipe;
		
		FInputPipe(FNewPipe)
			: FPipe{false}
		{
		}

		[[nodiscard]] void* NativeHandle() const
		{
			return ReadHandle;
		}

		[[nodiscard]] FString Read()
		{
			check(*this);
			return FPlatformProcess::ReadPipe(ReadHandle);
		}
		
		[[nodiscard]] operator FWriteHandle()&
		{
			return WriteHandle;
		}
	};
	
	struct [[nodiscard]] FOutputPipe: FPipe
	{
		using FPipe::FPipe;
		
		FOutputPipe(FNewPipe)
			: FPipe{true}
		{
		}
		
		[[nodiscard]] void* NativeHandle() const
		{
			return WriteHandle;
		}
		
		bool Write(const FString& Message)
		{
			check(*this);
			return FPlatformProcess::WritePipe(WriteHandle, Message);
		}
		
		[[nodiscard]] operator FReadHandle()&
		{
			return ReadHandle;
		}
	};
	
	struct [[nodiscard]] FProcess
	{
		constexpr FProcess() = default;
		FProcess(const FProcess&) = delete;
		FProcess& operator=(const FProcess&) = delete;
		
		constexpr FProcess(std::nullptr_t)
		{
		}
		
		FProcess(EProcessId ProcessId)
			: Handle{FPlatformProcess::OpenProcess(std::underlying_type_t<EProcessId>(ProcessId))}
		{
		}

		FProcess(const FProcessStartInfo& StartInfo)
			: Handle{FPlatformProcess::CreateProc(StartInfo).Get<FProcHandle>()}
		{
		}
		
		FProcess(const FProcessStartInfo& StartInfo, EProcessId& OutProcessId)
			: Handle{[&StartInfo, &OutProcessId]
			{
				TTuple<FProcHandle, EProcessId> Process = FPlatformProcess::CreateProc(StartInfo);
				OutProcessId = Process.Get<EProcessId>();
				return Process.Get<FProcHandle>();
			}()}
		{
		}
		
		constexpr FProcess(FProcess&& Other)
			: Handle{std::exchange(Other.Handle, {})}
		{
		}
		
		~FProcess()
		{
			FPlatformProcess::CloseProc(Handle);
		}
		
		FProcess& operator=(std::nullptr_t)
		{
			FPlatformProcess::CloseProc(Handle);
			return *this;
		}
		
		constexpr FProcess& operator=(FProcess&& Other)
		{
			Swap(Handle, Other.Handle);
			return *this;
		}
		
		[[nodiscard]] decltype(auto) NativeHandle() const
		{
			return Handle.Get();
		}
		
		[[nodiscard]] bool IsRunning()
		{
			check(*this);
			return FPlatformProcess::IsProcRunning(Handle);
		}
		
		[[nodiscard]] TOptional<int32> GetExitCode()
		{
			check(*this);
			int32 ExitCode;
			return FPlatformProcess::GetProcReturnCode(Handle, &ExitCode) ? TOptional{ExitCode} : NullOpt;
		}
		
		// Bug in PVS to flag references
		//-V:WaitForExit:530
		FProcess& WaitForExit()
		{
			check(*this);
			FPlatformProcess::WaitForProc(Handle);
			return *this;
		}
		
		// Bug in PVS to flag references
		//-V:Kill:530
		FProcess& Kill(bool bEntireTree = false)
		{
			check(*this);
			FPlatformProcess::TerminateProc(Handle, bEntireTree);
			return *this;
		}

		[[nodiscard]] bool operator==(std::nullptr_t) const
		{
			return !Handle.IsValid();
		}

		[[nodiscard]] explicit operator bool() const
		{
			return Handle.IsValid();
		}

	private:
		FProcHandle Handle;
	};
}
