// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

 // IWYU pragma: begin_exports

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformProperties.h"

// note that this is not defined to 1 like normal, because we don't want to have to define it to 0 whenever
// the Properties.h files are included in all other places, so just use #ifdef not #if in this special case
#define PROPERTY_HEADER_SHOULD_DEFINE_TYPE

#include COMPILED_PLATFORM_HEADER(PlatformProperties.h)

#undef PROPERTY_HEADER_SHOULD_DEFINE_TYPE

 // IWYU pragma: end_exports
