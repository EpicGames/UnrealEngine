// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformTLS.h"

#include COMPILED_PLATFORM_HEADER(PlatformTLS.h) // IWYU pragma: export

#if PLATFORM_THREAD_LOCAL_IS_UNSAFE_IN_MODULES

#include "HAL/TlsAutoCleanup.h"
#include "Misc/AssertionMacros.h"

namespace UE::Core::Private
{
	/**
	 * TModuleSafeThreadLocal is an alternative to thread_local that can safely be used in unloadable dynamic libraries on all platforms.
	 * Please see UE_MODULE_SAFE_THREAD_LOCAL for usage and limitations.
	 */
	template<class T>
	class TModuleSafeThreadLocal
	{
		static_assert(std::is_default_constructible_v<T>, "Thread-local object must be default-constructible");
		static_assert(std::is_copy_constructible_v<T>, "Thread-local object must be copy-constructible");

	public:
		UE_NONCOPYABLE(TModuleSafeThreadLocal);

		/** Constructor, allocates a TLS slot for this instance. */
		TModuleSafeThreadLocal()
			: TlsSlot(FPlatformTLS::AllocTlsSlot())
		{
			verify(TlsSlot != FPlatformTLS::InvalidTlsSlot);
		}

		/** Destructor, frees the TLS slot for this instance. */
		~TModuleSafeThreadLocal()
		{
			FPlatformTLS::FreeTlsSlot(TlsSlot);
		}

		/**
		 * Gets a thread-local instance of the value held, default-constructed on the first access.
		 *
		 * @return A reference to the per-thread instance of the value.
		 */
		T& Get()
		{
			TTlsAutoCleanupValue<T>* TlsValue = static_cast<TTlsAutoCleanupValue<T>*>(FPlatformTLS::GetTlsValue(TlsSlot));

			if (TlsValue == nullptr)
			{
				TlsValue = new TTlsAutoCleanupValue<T>(T());
				TlsValue->Register();
				FPlatformTLS::SetTlsValue(TlsSlot, TlsValue);
			}

			return TlsValue->Get();
		}

	private:

		uint32 TlsSlot;
	};
}

/**
 * Declares a variable Name of Type that acts thread_local in a way that is safe for use in unloadable dynamic libraries on all platforms.
 *
 * While we can and should use thread_local for TLS in most code, dynamic libraries designed to be unloaded cannot hold TLS objects on some platforms.
 * This is of particular concern when using merged modules on client builds, where thread_local usage in templates can lead to unintended TLS objects in libraries.
 *
 * Limitations:
 *	Usage of this class is subject to existing limits on platform TLS usage, so this should be used sparingly, only for code that has to be used in unloadable DLLs.
 *	The type used must be default-constructible to a known initial state, and copy-constructible.
 *
 * Example:
 *
 * 	struct FMyStruct {...};                                   // Will be default-constructed for each thread that uses it
 * 	UE_MODULE_SAFE_THREAD_LOCAL(FMyStruct, PerThreadMyStruct) // Each thread has its own PerThreadMyStruct instance
 */
#define UE_MODULE_SAFE_THREAD_LOCAL(Type, Name) \
	static UE::Core::Private::TModuleSafeThreadLocal<Type> TlsStorage##Name;\
	Type& Name = TlsStorage##Name.Get();

#else

// Identical to thread_local on platforms that do not opt into TModuleSafeThreadLocal
#define UE_MODULE_SAFE_THREAD_LOCAL(Type, Name) static thread_local Type Name;

#endif // PLATFORM_THREAD_LOCAL_IS_UNSAFE_IN_MODULES
