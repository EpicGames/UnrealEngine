// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlatformProcess.h"

// An RAII wrapper around FPlatformProcess::GetSynchEventFromPool/ReturnSynchEventToPool.
struct FPooledSyncEvent
{
	explicit FPooledSyncEvent(bool bIsManualReset = false)
	{
		Event = FPlatformProcess::GetSynchEventFromPool(bIsManualReset);
	}
	~FPooledSyncEvent()
	{
		*this = nullptr;
	}

	// Non-copyable
	FPooledSyncEvent(FPooledSyncEvent&&) = delete;
	FPooledSyncEvent(const FPooledSyncEvent&) = delete;
	FPooledSyncEvent& operator=(FPooledSyncEvent&&) = delete;
	FPooledSyncEvent& operator=(const FPooledSyncEvent&) = delete;

	explicit operator bool() const
	{
		return !!this->Event;
	}

	FEvent* operator->() const
	{
		return this->Event;
	}

	FPooledSyncEvent& operator=(TYPE_OF_NULLPTR)
	{
		if (Event)
		{
			FPlatformProcess::ReturnSynchEventToPool(Event);
			Event = nullptr;
		}
		return *this;
	}

	FEvent* Event;
};
