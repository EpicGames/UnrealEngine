// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Turns an preprocessor token into a real string (see UBT_COMPILED_PLATFORM)
#define UE_STRINGIZE(Token) UE_PRIVATE_STRINGIZE(Token)
#define UE_PRIVATE_STRINGIZE(Token) #Token

// Concatenates two or more preprocessor tokens, performing macro expansion on them first
#define UE_JOIN(TokenA, TokenB, ...) UE_APPEND_VA_ARG_COUNT(UE_PRIVATE_JOIN, __VA_ARGS__)(TokenA, TokenB, __VA_ARGS__)
#define UE_PRIVATE_JOIN0(TokenA, TokenB) TokenA##TokenB
#define UE_PRIVATE_JOIN1(TokenA, TokenB, TokenC) TokenA##TokenB##TokenC
#define UE_PRIVATE_JOIN2(TokenA, TokenB, TokenC, TokenD) TokenA##TokenB##TokenC##TokenD
#define UE_PRIVATE_JOIN3(TokenA, TokenB, TokenC, TokenD, TokenE) TokenA##TokenB##TokenC##TokenD##TokenE
#define UE_PRIVATE_JOIN4(TokenA, TokenB, TokenC, TokenD, TokenE, TokenF) TokenA##TokenB##TokenC##TokenD##TokenE##TokenF

// Concatenates the first two preprocessor tokens of a variadic list, after performing macro expansion on them
#define UE_JOIN_FIRST(Token, ...) UE_PRIVATE_JOIN_FIRST(Token, __VA_ARGS__)
#define UE_PRIVATE_JOIN_FIRST(Token, ...) Token##__VA_ARGS__

// Expands to the second argument or the third argument if the first argument is 1 or 0 respectively
#define UE_IF(OneOrZero, Token1, Token0) UE_JOIN(UE_PRIVATE_IF_, OneOrZero)(Token1, Token0)
#define UE_PRIVATE_IF_1(Token1, Token0) Token1
#define UE_PRIVATE_IF_0(Token1, Token0) Token0

// Expands to the parameter list of the macro - used to pass a *potentially* comma-separated identifier to another macro as a single parameter
#define UE_COMMA_SEPARATED(First, ...) First, ##__VA_ARGS__

// Expands to a number which is the count of variadic arguments passed to it.
#define UE_VA_ARG_COUNT(...) UE_APPEND_VA_ARG_COUNT(, ##__VA_ARGS__)

// Expands to a token of Prefix##<count>, where <count> is the number of variadic arguments.
//
// Example:
//   UE_APPEND_VA_ARG_COUNT(SOME_MACRO_)          => SOME_MACRO_0
//   UE_APPEND_VA_ARG_COUNT(SOME_MACRO_, a, b, c) => SOME_MACRO_3
#if !defined(_MSVC_TRADITIONAL) || !_MSVC_TRADITIONAL
	#define UE_APPEND_VA_ARG_COUNT(Prefix, ...) UE_PRIVATE_APPEND_VA_ARG_COUNT(Prefix, ##__VA_ARGS__, 26, 25, 24, 23, 22, 21, 20, 19, 18, 17, 16, 15, 14, 13, 12, 11, 10, 9, 8, 7, 6, 5, 4, 3, 2, 1, 0)
#else
	#define UE_APPEND_VA_ARG_COUNT(Prefix, ...) UE_PRIVATE_APPEND_VA_ARG_COUNT_INVOKE(UE_PRIVATE_APPEND_VA_ARG_COUNT, (Prefix, ##__VA_ARGS__, 26, 25, 24, 23, 22, 21, 20, 19, 18, 17, 16, 15, 14, 13, 12, 11, 10, 9, 8, 7, 6, 5, 4, 3, 2, 1, 0))

	// MSVC's traditional preprocessor doesn't handle the zero-argument case correctly, so we use a workaround.
	// The workaround uses token pasting of Macro##ArgsInParens, which the conformant preprocessor doesn't like and emits C5103.
	#define UE_PRIVATE_APPEND_VA_ARG_COUNT_INVOKE(Macro, ArgsInParens) UE_PRIVATE_APPEND_VA_ARG_COUNT_EXPAND(Macro##ArgsInParens)
	#define UE_PRIVATE_APPEND_VA_ARG_COUNT_EXPAND(Arg) Arg
#endif
#define UE_PRIVATE_APPEND_VA_ARG_COUNT(Prefix,A,B,C,D,E,F,G,H,I,J,K,L,M,N,O,P,Q,R,S,T,U,V,W,X,Y,Z,Count,...) Prefix##Count

// Expands to nothing - used as a placeholder
#define UE_EMPTY

// Expands to nothing when used as a function - used as a placeholder
#define UE_EMPTY_FUNCTION(...)

// Expands to a comma to allow passing a comma as a parameter to a macro
#define UE_COMMA ,

// Expands to a left parenthesis.
#define UE_LPAREN (
// Expands to a right parenthesis 
#define UE_RPAREN )
// Expands the arguments within parentheses. 
#define UE_ADD_PARENS(...) (__VA_ARGS__)

// Removes a single layer of parentheses from a macro argument if they are present - used to allow
// brackets to be optionally added when the argument contains commas, e.g.:
//
// #define DEFINE_VARIABLE(Type, Name) UE_REMOVE_OPTIONAL_PARENS(Type) Name;
//
// DEFINE_VARIABLE(int, IntVar)                  // expands to: int IntVar;
// DEFINE_VARIABLE((TPair<int, float>), PairVar) // expands to: TPair<int, float> PairVar;
#define UE_REMOVE_OPTIONAL_PARENS(...) UE_JOIN_FIRST(UE_PRIVATE_PREPROCESSOR_REMOVE_OPTIONAL_PARENS,UE_PRIVATE_PREPROCESSOR_REMOVE_OPTIONAL_PARENS __VA_ARGS__)
#define UE_PRIVATE_PREPROCESSOR_REMOVE_OPTIONAL_PARENS(...) UE_PRIVATE_PREPROCESSOR_REMOVE_OPTIONAL_PARENS __VA_ARGS__
#define UE_PRIVATE_PREPROCESSOR_REMOVE_OPTIONAL_PARENSUE_PRIVATE_PREPROCESSOR_REMOVE_OPTIONAL_PARENS

// Removes a single layer of parentheses from a macro argument unconditionally
// Note that this macro accepts any number of arguments but will only remove parentheses from the first
// e.g. UE_REMOVE_PARENS((1,2), (3,4)) will expand to 1,2,(3,4)
#define UE_REMOVE_PARENS(...) UE_PRIVATE_REMOVE_PARENS __VA_ARGS__
#define UE_PRIVATE_REMOVE_PARENS(...) __VA_ARGS__

// Expand the given macro with each following parenthesized argument or set of arguments, with an optional separator.
// Supports up to 10 arguments at time of writing. 
// The primary use case is to write macros that can expand their variadic arguments to produce a variable number of 
// expressions or statements.
//
// e.g. 
// #define DECLARE_INT_VARIABLE(Name) int Name;
// #define DECLARE_TYPE(TypeName, ...) struct TypeName { UE_FOR_EACH(DECLARE_INT_VARIABLE, ## __VA_ARGS__) };
// DECLARE_TYPE(Foo, (One), (Two), (Three))
// expands to
// struct Foo { int One; int Two; int Three; };
// 
// To expand macro arguments as a comma-separated list, use UE_COMMA as the separator
// e.g.
//  SomeFunction(UE_FOR_EACH_WITH_SEPARATOR(PASSTHROUGH, UE_COMMA, ## __VA_ARGS__))
// may expand to to
//  SomeFunction(One, Two, Three)
#define UE_FOR_EACH(Macro, ...) UE_FOR_EACH_WITH_SEPARATOR(Macro, UE_EMPTY, ## __VA_ARGS__)
#define UE_FOR_EACH_WITH_SEPARATOR(Macro, Separator, ...) UE_APPEND_VA_ARG_COUNT(UE_PRIVATE_FOR_EACH_, ## __VA_ARGS__)(Macro, UE_ADD_PARENS(Separator), ## __VA_ARGS__)

#define UE_PRIVATE_FOR_EACH_0(Macro, Separator)
#define UE_PRIVATE_FOR_EACH_1(Macro, Separator, A1) Macro A1
#define UE_PRIVATE_FOR_EACH_2(Macro, Separator, A1, A2) Macro A1 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A2
#define UE_PRIVATE_FOR_EACH_3(Macro, Separator, A1, A2, A3) Macro A1 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A2 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A3
#define UE_PRIVATE_FOR_EACH_4(Macro, Separator, A1, A2, A3, A4) Macro A1 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A2 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A3 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A4
#define UE_PRIVATE_FOR_EACH_5(Macro, Separator, A1, A2, A3, A4, A5) Macro A1 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A2 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A3 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A4 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A5
#define UE_PRIVATE_FOR_EACH_6(Macro, Separator, A1, A2, A3, A4, A5, A6) Macro A1 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A2 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A3 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A4 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A5 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A6
#define UE_PRIVATE_FOR_EACH_7(Macro, Separator, A1, A2, A3, A4, A5, A6, A7) Macro A1 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A2 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A3 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A4 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A5 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A6 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A7
#define UE_PRIVATE_FOR_EACH_8(Macro, Separator, A1, A2, A3, A4, A5, A6, A7, A8) Macro A1 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A2 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A3 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A4 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A5 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A6 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A7 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A8
#define UE_PRIVATE_FOR_EACH_9(Macro, Separator, A1, A2, A3, A4, A5, A6, A7, A8, A9) Macro A1 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A2 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A3 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A4 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A5 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A6 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A7 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A8 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A9
#define UE_PRIVATE_FOR_EACH_10(Macro, Separator, A1, A2, A3, A4, A5, A6, A7, A8, A9, A10) Macro A1 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A2 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A3 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A4 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A5 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A6 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A7 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A8 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A9 UE_REMOVE_OPTIONAL_PARENS(Separator) Macro A10

// Performs macro expansion on the given arguments which will allow the preprocessor to then apply further macro
// expansion to the result.
// Used in cases where expansion needs to be performed for the preprocessor to recognize the first parameter as a
// function macro rather than an object macro, or cases where expansion must be performed to produce the correct
// number of arguments for a macro
// 
// e.g.
// #define ADD(a, b) a + b
//
// #define ARGS (1, 2)
//
// const TCHAR Str1[] = UE_STRINGIZE(ADD ARGS); // "ADD (1, 2)"
// const TCHAR Str2[] = UE_STRINGIZE(UE_EXPAND(ADD ARGS)); // "1 + 2"
// 
// UE_ADD_PARENS can also be used
// UE_EXPAND(MYMACRO UE_ADD_PARENS(UE_REMOVE_OPTIONAL_PARENS(FIRST), UE_REMOVE_OPTIONAL_PARENS(SECOND)))
#define UE_EXPAND(...) __VA_ARGS__

// setup standardized way of including platform headers from the "uber-platform" headers like PlatformFile.h
#ifdef OVERRIDE_PLATFORM_HEADER_NAME
// allow for an override, so compiled platforms Win64 and Win32 will both include Windows
#define PLATFORM_HEADER_NAME OVERRIDE_PLATFORM_HEADER_NAME
#else
// otherwise use the compiled platform name
#define PLATFORM_HEADER_NAME UBT_COMPILED_PLATFORM
#endif

#define UE_SOURCE_LOCATION TEXT(__FILE__ "(" UE_STRINGIZE(__LINE__) ")")

#ifndef PLATFORM_IS_EXTENSION
#define PLATFORM_IS_EXTENSION 0
#endif

#if PLATFORM_IS_EXTENSION
// Creates a string that can be used to include a header in the platform extension form "PlatformHeader.h", not like
// below form. When using this you should add "// IWYU pragma: export" at the end of the line.
#define COMPILED_PLATFORM_HEADER(Suffix) UE_STRINGIZE(UE_JOIN(PLATFORM_HEADER_NAME, Suffix))
#else
// Creates a string that can be used to include a header in the form "Platform/PlatformHeader.h", like
// "Windows/WindowsPlatformFile.h". When using this you should add "// IWYU pragma: export" at the end of the line.
#define COMPILED_PLATFORM_HEADER(Suffix) UE_STRINGIZE(UE_JOIN(PLATFORM_HEADER_NAME/PLATFORM_HEADER_NAME, Suffix))
#endif

// Creates a string that can be used to include a header in the platform extension form "PlatformHeader.h", but will
// not include a directory like COMPILED_PLATFORM_HEADER does, generally for UBT generated headers.
#define COMPILED_PLATFORM_HEADER_GENERATED(Suffix) UE_STRINGIZE(UE_JOIN(PLATFORM_HEADER_NAME, Suffix))

#if PLATFORM_IS_EXTENSION
// Creates a string that can be used to include a header with the platform in its name, like
// "Prefix/PlatformNameSuffix.h". When using this you should add "// IWYU pragma: export" at the end of the line.
#define COMPILED_PLATFORM_HEADER_WITH_PREFIX(Prefix, Suffix) UE_STRINGIZE(Prefix/UE_JOIN(PLATFORM_HEADER_NAME, Suffix))
#else
// Creates a string that can be used to include a header with the platform in its name, like
// "Prefix/PlatformName/PlatformNameSuffix.h". When using this you should add "// IWYU pragma: export" at the end of the
// line.
#define COMPILED_PLATFORM_HEADER_WITH_PREFIX(Prefix, Suffix) UE_STRINGIZE(Prefix/PLATFORM_HEADER_NAME/UE_JOIN(PLATFORM_HEADER_NAME, Suffix))
#endif

// These macros should be regarded as deprecated - use the UE_ macros they map to instead.
#define PREPROCESSOR_TO_STRING(Token)                 UE_DEPRECATED_MACRO(5.8, "PREPROCESSOR_TO_STRING has been deprecated - please use UE_STRINGIZE instead.") UE_STRINGIZE(Token)
#define PREPROCESSOR_JOIN(TokenA, TokenB)             UE_DEPRECATED_MACRO(5.8, "PREPROCESSOR_JOIN has been deprecated - please use UE_JOIN instead.") UE_JOIN(TokenA, TokenB)
#define PREPROCESSOR_JOIN_FIRST(Token, ...)           UE_DEPRECATED_MACRO(5.8, "PREPROCESSOR_JOIN_FIRST has been deprecated - please use UE_JOIN_FIRST instead.") UE_JOIN_FIRST(Token, ##__VA_ARGS__)
#define PREPROCESSOR_IF(OneOrZero, Token1, Token0)    UE_DEPRECATED_MACRO(5.8, "PREPROCESSOR_IF has been deprecated - please use UE_IF instead.") UE_IF(OneOrZero, Token1, Token0)
#define PREPROCESSOR_COMMA_SEPARATED(First, ...)      UE_DEPRECATED_MACRO(5.8, "PREPROCESSOR_COMMA_SEPARATED has been deprecated - please use UE_COMMA_SEPARATED instead.") UE_COMMA_SEPARATED(First, ##__VA_ARGS__)
#define PREPROCESSOR_VA_ARG_COUNT(...)                UE_DEPRECATED_MACRO(5.8, "PREPROCESSOR_VA_ARG_COUNT has been deprecated - please use UE_VA_ARG_COUNT instead.") UE_VA_ARG_COUNT(__VA_ARGS__)
#define PREPROCESSOR_APPEND_VA_ARG_COUNT(Prefix, ...) UE_DEPRECATED_MACRO(5.8, "PREPROCESSOR_APPEND_VA_ARG_COUNT has been deprecated - please use UE_APPEND_VA_ARG_COUNT instead.") UE_APPEND_VA_ARG_COUNT(Prefix, ##__VA_ARGS__)
#define PREPROCESSOR_NOTHING                          UE_DEPRECATED_MACRO(5.8, "PREPROCESSOR_NOTHING has been deprecated - please use UE_EMPTY instead.") UE_EMPTY
#define PREPROCESSOR_NOTHING_FUNCTION(...)            UE_DEPRECATED_MACRO(5.8, "PREPROCESSOR_NOTHING_FUNCTION has been deprecated - please use UE_EMPTY_FUNCTION instead.") UE_EMPTY_FUNCTION(__VA_ARGS__)
#define PREPROCESSOR_REMOVE_OPTIONAL_PARENS(...)      UE_DEPRECATED_MACRO(5.8, "PREPROCESSOR_REMOVE_OPTIONAL_PARENS has been deprecated - please use UE_REMOVE_OPTIONAL_PARENS instead.") UE_REMOVE_OPTIONAL_PARENS(__VA_ARGS__)
