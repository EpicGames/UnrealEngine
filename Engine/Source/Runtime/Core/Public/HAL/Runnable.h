// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Interface for "runnable" objects.
 *
 * A runnable object is an object that is "run" on an arbitrary thread. The call usage pattern is
 * Init(), Run(), Exit(). The thread that is going to "run" this object always uses those calling
 * semantics. It does this on the thread that is created so that any thread specific uses (TLS, etc.)
 * are available in the contexts of those calls. A "runnable" does all initialization in Init().
 *
 * If initialization fails, the thread stops execution and returns an error code. If it succeeds,
 * Run() is called where the real threaded work is done. Upon completion, Exit() is called to allow
 * correct clean up.
 */
class FRunnable
{
public:

	/**
	 * Initializes the runnable object.
	 *
	 * This method is called in the context of the thread object that aggregates this, not the
	 * thread that passes this runnable to a new thread.
	 *
	 * @return True if initialization was successful, false otherwise
	 * @see Run, Stop, Exit
	 */
	virtual bool Init()
	{
		return true;
	}

	/**
	 * Runs the runnable object.
	 *
	 * This is where all per object thread work is done. This is only called if the initialization was successful.
	 *
	 * @return The exit code of the runnable object
	 * @see Init, Stop, Exit
	 */
	virtual uint32 Run() = 0;

	/**
	 * Stops the runnable object.
	 *
	 * This is called if a thread is requested to terminate early.
	 * @see Init, Run, Exit
	 */
	virtual void Stop() { }

	/**
	 * Exits the runnable object.
	 *
	 * Called in the context of the aggregating thread to perform any cleanup.
	 * @see Init, Run, Stop
	 */
	virtual void Exit() { }

	/**
	 * Gets single thread interface pointer used for ticking this runnable when multi-threading is disabled.
	 * If the interface is not implemented, this runnable will not be ticked when FPlatformProcess::SupportsMultithreading() is false.
	 *
	 * @return Pointer to the single thread interface or nullptr if not implemented.
	 */
	virtual class FSingleThreadRunnable* GetSingleThreadInterface( )
	{
		return nullptr;
	}

	/** Virtual destructor */
	virtual ~FRunnable() = default;
};
