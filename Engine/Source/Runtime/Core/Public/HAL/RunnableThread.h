// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformTLS.h"
#include "HAL/PlatformAffinity.h"

class FRunnable;
class FTlsAutoCleanup;

/**
 * Interface for runnable threads.
 *
 * This interface specifies the methods used to manage a thread's life cycle.
 */
class FRunnableThread
{
	friend class FThreadSingletonInitializer;
	friend class FTlsAutoCleanup;
	friend class FThreadManager;
	friend class FForkableThread;
	friend class FForkProcessHelper;

	/** Index of TLS slot for FRunnableThread pointer. */
	static CORE_API uint32 RunnableTlsSlot;

public:

	/** Gets a new Tls slot for storing the runnable thread pointer. */
	static CORE_API uint32 GetTlsSlot();

	/**
	 * Factory method to create a thread with the specified stack size and thread priority.
	 *
	 * @param InRunnable The runnable object to execute
	 * @param ThreadName Name of the thread
	 * @param InStackSize The size of the stack to create. 0 means use the current thread's stack size
	 * @param InThreadPri Tells the thread whether it needs to adjust its priority or not. Defaults to normal priority
	 * @return The newly created thread or nullptr if it failed
	 */
	static CORE_API FRunnableThread* Create(
		class FRunnable* InRunnable,
		const TCHAR* ThreadName,
		uint32 InStackSize = 0,
		EThreadPriority InThreadPri = TPri_Normal,
		uint64 InThreadAffinityMask = FPlatformAffinity::GetNoAffinityMask(),
		EThreadCreateFlags InCreateFlags = EThreadCreateFlags::None);

	/**
	 * Changes the thread priority of the currently running thread
	 *
	 * @param NewPriority The thread priority to change to
	 */
	virtual void SetThreadPriority( EThreadPriority NewPriority ) = 0;

	/**
	* Changes the thread affinity of the currently running thread
	*
	* @param ThreadAffinityMask The thread affinity to change to (can be 0 to keep previously set affinity mask)
	* @param ProcessorGroup The thread group to change to
	* @return returns true if the affinity changed, false if affinity did not change
	*/
	virtual bool SetThreadAffinity( const FThreadAffinity& Affinity ) { return false; };

	/**
	 * Tells the thread to either pause execution or resume depending on the
	 * passed in value.
	 *
	 * @param bShouldPause Whether to pause the thread (true) or resume (false)
	 */
	virtual void Suspend( bool bShouldPause = true ) = 0;

	/**
	 * Tells the thread to exit. If the caller needs to know when the thread has exited, it should use the bShouldWait value.
	 * It's highly recommended not to kill the thread without waiting for it.
	 * Having a thread forcibly destroyed can cause leaks and deadlocks.
	 * 
	 * The kill method is calling Stop() on the runnable to kill the thread gracefully.
	 * 
	 * @param bShouldWait	If true, the call will wait infinitely for the thread to exit.					
	 * @return Always true
	 */
	virtual bool Kill( bool bShouldWait = true ) = 0;

	/** Halts the caller until this thread is has completed its work. */
	virtual void WaitForCompletion() = 0;

	/** List of unique thread types we can create */
	enum class ThreadType
	{
		// Regular thread that executes the runnable object in it's own context
		Real,
		// Fake threads are created for a single threaded environment and are always executed from the main tick
		Fake,
		// Forkable threads will behave like fake threads for the master process, but will become real threads on forked processes
		Forkable,
	};

	/** Returns the type of thread this is */
	virtual FRunnableThread::ThreadType GetThreadType() const
	{
		return ThreadType::Real;
	}

	/**
	 * Thread ID for this thread 
	 *
	 * @return ID that was set by CreateThread
	 * @see GetThreadName
	 */
	const uint32 GetThreadID() const
	{
		return ThreadID;
	}

	/**
	 * Retrieves the given name of the thread
	 *
	 * @return Name that was set by CreateThread
	 * @see GetThreadID
	 */
	const FString& GetThreadName() const
	{
		return ThreadName;
	}

	/** Returns the runnable's thread priority */
	EThreadPriority GetThreadPriority() const
	{
		return ThreadPriority;
	}

	/** Default constructor. */
	CORE_API FRunnableThread();

	/** Virtual destructor */
	CORE_API virtual ~FRunnableThread();

	/**
	 * @return a runnable thread that is executing this runnable, if return value is nullptr, it means the running thread can be game thread or a thread created outside the runnable interface
	 */
	static FRunnableThread* GetRunnableThread()
	{
		FRunnableThread* RunnableThread = (FRunnableThread*)FPlatformTLS::GetTlsValue(RunnableTlsSlot);
		return RunnableThread;
	}

protected:

	/**
	 * Creates the thread with the specified stack size and thread priority.
	 *
	 * @param InRunnable The runnable object to execute
	 * @param ThreadName Name of the thread
	 * @param InStackSize The size of the stack to create. 0 means use the current thread's stack size
	 * @param InThreadPri Tells the thread whether it needs to adjust its priority or not. Defaults to normal priority
	 * @return True if the thread and all of its initialization was successful, false otherwise
	 */
	virtual bool CreateInternal( FRunnable* InRunnable, const TCHAR* InThreadName,
		uint32 InStackSize = 0,
		EThreadPriority InThreadPri = TPri_Normal, uint64 InThreadAffinityMask = 0, 
		EThreadCreateFlags InCreateFlags = EThreadCreateFlags::None) = 0;

	/** Stores this instance in the runnable thread TLS slot. */
	CORE_API void SetTls();

	/** Deletes all FTlsAutoCleanup objects created for this thread. */
	CORE_API void FreeTls();

	/** Holds the name of the thread. */
	FString ThreadName;

	/** The runnable object to execute on this thread. */
	FRunnable* Runnable;

	/** Sync event to make sure that Init() has been completed before allowing the main thread to continue. */
	FEvent* ThreadInitSyncEvent;

	/** The Affinity to run the thread with. */
	uint64 ThreadAffinityMask;

	/** The priority to run the thread at. */
	EThreadPriority ThreadPriority;

	/** ID set during thread creation. */
	uint32 ThreadID;

private:

	/** Called to setup a newly created RunnableThread */
	static CORE_API void SetupCreatedThread(FRunnableThread*& NewThread, class FRunnable* InRunnable, const TCHAR* ThreadName, uint32 InStackSize, EThreadPriority InThreadPri, uint64 InThreadAffinityMask, EThreadCreateFlags InCreateFlags);

	/** Used by the thread manager to tick threads in single-threaded mode */
	virtual void Tick() {}

	/**
	 * Called on the parent process before the fork occurs
	 */
	virtual void OnPreFork()
	{
		checkf(false, TEXT("Only forkable threads should receive OnPreFork."));
	}

	/**
	 * Called on the forked process when the forkable thread can create a real thread
	 */
	virtual void OnPostFork()
	{
		checkf(false, TEXT("Only forkable threads should receive OnPostFork."));
	}

	/**
	 * Called after the internal thread is created so it can register debug information
	 */
	CORE_API void PostCreate(EThreadPriority ThreadPriority);
};
