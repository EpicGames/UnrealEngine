// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HAL/PlatformAffinity.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

/**
 * Simple API for system threads. 
 * Before using, please make sure you really need a new system thread. By default and in the majority of cases
 * parallel processing should be done by TaskGraph.
 * For richer functionality check `FRunnable`/`FRunnableThread`.
 * It's up to user to provide a way to notify the thread function to exit on demand.
 * Before destroying the instance it must be either `Join`ed or `Detach`ed.
 * Example:
 *		FThread Thread{TEXT("New thread"), []() { do_something_important(); }};
 *		// ... continue in the caller thread
 *		Thread.Join();
 * For more verbose example check `TestTypicalUseCase` in `ThreadTest.cpp`
 */
class FThread final
{
public:
	// indicates if the thread should be forked in case the owning process is forked
	enum EForkable { Forkable, NonForkable };

	/**
	 * Creates new "empty" thread object that doesn't represent a system thread
	 */
	FThread()
	{}

	/**
	* Creates and immediately starts a new system thread that will execute `ThreadFunction` argument.
	* Can return before the thread is actually started or when it already finished execution.
	* @param ThreadName Name of the thread
	* @param ThreadFunction The function that will be executed by the newly created thread
	* @param StackSize The size of the stack to create. 0 means use the current thread's stack size
	* @param ThreadPriority Tells the thread whether it needs to adjust its priority or not. Defaults to normal priority
	* @param ThreadAffinity Tells the thread whether it needs to adjust its affinity or not. Defaults to no affinity
	* @param IsForkable Tells the thread whether it can be forked. Defaults to NonForkable
	*/
	CORE_API FThread(
		TCHAR const* ThreadName,
		TUniqueFunction<void()>&& ThreadFunction,
		uint32 StackSize = 0,
		EThreadPriority ThreadPriority = TPri_Normal,
		FThreadAffinity ThreadAffinity = FThreadAffinity(),
		EForkable IsForkable = NonForkable
	);

	// with SingleThreadTickFunction that will be executed every frame if running with `-nothreading
	// (FPlatformProcess::SupportsMultithreading() == false)
	CORE_API FThread(
		TCHAR const* ThreadName,
		TUniqueFunction<void()>&& ThreadFunction,
		TUniqueFunction<void()>&& SingleThreadTickFunction,
		uint32 StackSize = 0,
		EThreadPriority ThreadPriority = TPri_Normal,
		FThreadAffinity ThreadAffinity = FThreadAffinity(),
		EForkable IsForkable = NonForkable
	);

	// non-copyable
	FThread(const FThread&) = delete;
	FThread& operator=(const FThread&) = delete;

	FThread(FThread&&) = default;
	/**
	 * Move assignment operator.
	 * Asserts if the instance is joinable.
	 */
	CORE_API FThread& operator=(FThread&& Other);

	/**
	 * Destructor asserts if the instance is not joined or detached.
	 */
	CORE_API ~FThread();

	/**
	 * Checks if the thread object identifies an active thread of execution. 
	 * A thread that has finished executing code, but has not yet been joined is still considered an active 
	 * thread of execution and is therefore joinable.
	 * @see Join
	 */
	CORE_API bool IsJoinable() const;

	/**
	 * Blocks the current thread until the thread identified by `this` finishes its execution.
	 * The completion of the thread identified by `this` synchronizes with the corresponding successful return from Join().
	 * No synchronization is performed on `this` itself. Concurrently calling Join() on the same FThread object 
	 * from multiple threads constitutes a data race that results in undefined behavior.
	 * @see IsJoinable
	 */
	CORE_API void Join();

	static constexpr uint32 InvalidThreadId = ~uint32(0);

	/**
	 * @return Thread ID for this thread
	 */
	CORE_API uint32 GetThreadId() const;

#if 0 // disabled as it doesn't work as intended

	/**
	 * Separates the thread of execution from the thread object, allowing execution to continue independently. 
	 * Any allocated resources will be freed once the thread exits.
	 * After calling detach `this` no longer owns any thread.
	 */
	CORE_API void Detach();

#endif

private:
	TSharedPtr<class FThreadImpl, ESPMode::ThreadSafe> Impl; // "shared" with `FThreadImpl::Self`
};
