// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h" // for DECLARE_DELEGATE_OneParam
#include "HAL/CriticalSection.h"
#include "HAL/Runnable.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/ThreadSafeCounter.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"

#if PLATFORM_UNIX
#include "Unix/UnixSignalHeartBeat.h"
#endif

DECLARE_DELEGATE_OneParam(FOnThreadStuck, uint32);
DECLARE_DELEGATE_OneParam(FOnThreadUnstuck, uint32);
DECLARE_DELEGATE_OneParam(FOnHangDelegate, uint32);

/**
 * Our own local clock.
 * Platforms that support suspend/resume have problems where a suspended title acts like
 * a long hitch, causing the hang detector to fire incorrectly when the title is resumed.
 *
 * To solve this, we accumulate our own time on the hang detector thread.
 * When the title is suspended, this thread is also suspended, and the local clock stops.
 * The delta is clamped so if we are resumed, the clock continues from where it left off.
 */
class FThreadHeartBeatClock
{
	uint64 CurrentCycles;
	uint64 LastRealTickCycles;
	const uint64 MaxTimeStepCycles;

public:
	CORE_API FThreadHeartBeatClock(double InMaxTimeStep);

	CORE_API void Tick();
	CORE_API double Seconds();
};

/**
 * Thread heartbeat check class.
 * Used by crash handling code to check for hangs.
 */
class FThreadHeartBeat : public FRunnable
{
	static CORE_API FThreadHeartBeat* Singleton;

	/** Holds per-thread info about the heartbeat */
	struct FHeartBeatInfo
	{
		FHeartBeatInfo()
			: LastHeartBeatTime(0.0)
			, LastHangTime(0.0)
			, SuspendedCount(0)
			, HangDuration(0)
			, LastStuckTime(0.0)
			, StuckDuration(0.0)
			, HeartBeatName()
		{}

		/** Time we last received a heartbeat for the current thread */
		double LastHeartBeatTime;
		/** Time we last detected a hang due to lack of heartbeats for the current thread */
		double LastHangTime;
		/** Suspended counter */
		int32 SuspendedCount;
		/** The timeout for this thread */
		double HangDuration;

		/** Time we last detected thread stuck due to lack of heartbeats for the current thread */
		double LastStuckTime;
		/** How long it's benn stuck thread */
		double StuckDuration;
		/** An optional FName */
		FName HeartBeatName;

		/** Suspends this thread's heartbeat */
		void Suspend()
		{
			SuspendedCount++;
		}
		/** Resumes this thread's heartbeat */
		void Resume(double CurrentTime)
		{
			check(SuspendedCount > 0);
			if (--SuspendedCount == 0)
			{
				LastHeartBeatTime = CurrentTime;
			}
		}
	};
	/** Thread to run the worker FRunnable on */
	FRunnableThread* Thread;
	/** Stops this thread */
	FThreadSafeCounter StopTaskCounter;
	/** Synch object for the heartbeat */
	FCriticalSection HeartBeatCritical;
	/** Keeps track of the last heartbeat time for threads */
	TMap<uint32, FHeartBeatInfo> ThreadHeartBeat;
	/** The last heartbeat time for the rendering or RHI thread frame present. */
	FHeartBeatInfo PresentHeartBeat;

	/** Synch object for the function heartbeat */
	FCriticalSection FunctionHeartBeatCritical;
	/** Keeps track of the last heartbeat time for a function, can't be nested */
	TMap<uint32, FHeartBeatInfo> FunctionHeartBeat;

	/** Synch object for the checkpoint heartbeat */
	FCriticalSection CheckpointHeartBeatCritical;
	/** Keeps track of the last heartbeat time for a checkpoint */
	TMap<FName, FHeartBeatInfo> CheckpointHeartBeat;

	/** True if heartbeat should be measured */
	FThreadSafeBool bReadyToCheckHeartbeat;
	/** Max time the thread is allowed to not send the heartbeat*/
	double ConfigHangDuration;
	double CurrentHangDuration;
	double ConfigPresentDuration;
	double CurrentPresentDuration;
	double ConfigStuckDuration;
	double CurrentStuckDuration;

	double HangDurationMultiplier;

	/** CRC of the last hang's callstack */
	uint32 LastHangCallstackCRC;
	/** Id of the last thread that hung */
	uint32 LastHungThreadId;

	uint32 LastStuckThreadId;

	bool bHangsAreFatal;

	FThreadSafeCounter GlobalSuspendCount;
	int32 CheckpointSuspendCount;

	FThreadHeartBeatClock Clock;

	FOnThreadStuck OnStuck;
	FOnThreadUnstuck OnUnstuck;
	FOnHangDelegate OnHangDelegate;

	CORE_API FThreadHeartBeat();
	CORE_API virtual ~FThreadHeartBeat();

	CORE_API void FORCENOINLINE OnHang(double HangDuration, uint32 ThreadThatHung);
	CORE_API void FORCENOINLINE OnPresentHang(double HangDuration);
	CORE_API void FORCENOINLINE OnGPUHang(double HangDuration);

	CORE_API bool IsEnabled();

	void InitSettingsInternal();

public:

	enum EConstants
	{
		/** Invalid thread Id used by CheckHeartBeat */
		InvalidThreadId = (uint32)-1,

		/** Id used to track presented frames (supported platforms only). */
		PresentThreadId = (uint32)-2
	};

	/** Gets the heartbeat singleton */
	static CORE_API FThreadHeartBeat& Get();
	static CORE_API FThreadHeartBeat* GetNoInit();

	/** Begin measuring heartbeat */
	CORE_API void Start();
	/** Update settings from command line or ini */
	CORE_API void InitSettings();
	/** Called from a thread once per frame to update the heartbeat time */
	UE_DEPRECATED(5.7, "HeartBeat(bool bReadConfig) is deprecated, please use HeartBeat() instead.")
	CORE_API void HeartBeat(bool bReadConfig);
	CORE_API void HeartBeat();
	/** Called from the rendering or RHI thread when the platform RHI presents a frame (supported platforms only). */
	CORE_API void PresentFrame();
	/** Called by a supervising thread to check the threads' health */
	CORE_API uint32 CheckHeartBeat(double& OutHangDuration);
	/** Called by a thread when it's no longer expecting to be ticked */
	CORE_API void KillHeartBeat();

	/** Called from a thread once on entry to a function to be monitored */
	CORE_API void MonitorFunctionStart();
	/** Called by a thread when a function has completed and no longer needs to be monitored */
	CORE_API void MonitorFunctionEnd();
	/** Called by a supervising thread to check all function calls' being monitored health */
	CORE_API uint32 CheckFunctionHeartBeat(double& OutHangDuration);

	/* 
		Called from a thread to register a checkpoint to be monitored 
		@param EndCheckPoint name of the checkpoint that needs to be reached. TimeToReachCheckPoint the time duration we have to reach the specified checkpoint.
	*/
	CORE_API void MonitorCheckpointStart(FName EndCheckPoint, double TimeToReachCheckpoint);
	/* Called from a thread when a checkpoint has ended */
	CORE_API void MonitorCheckpointEnd(FName CheckPoint);
	/* Called by a supervising thread to check all checkpoints forward progress */
	CORE_API uint32 CheckCheckpointHeartBeat(double& OutHangDuration);

	/** 
	 * Suspend heartbeat measuring for the current thread if the thread has already had a heartbeat 
	 * @param bAllThreads If true, suspends heartbeat for all threads, not only the current one
	 */
	CORE_API void SuspendHeartBeat(bool bAllThreads = false);
	/** 
	 * Resume heartbeat measuring for the current thread 
	 * @param bAllThreads If true, resumes heartbeat for all threads, not only the current one
	 */
	CORE_API void ResumeHeartBeat(bool bAllThreads = false);

	/**
	* Returns true/false if this thread is currently performing heartbeat monitoring
	*/
	CORE_API bool IsBeating();

	/** 
	 * Sets a multiplier to the hang duration (>= 1.0).
	 * Can be used to extend the duration during loading screens etc.
	 */
	CORE_API void SetDurationMultiplier(double NewMultiplier);

	/*
	* Get the Id of the last thread to trigger the hang detector.
	* Returns InvalidThreadId if hang detector has not been triggered.
	*/
	uint32 GetLastHungThreadId() const { return LastHungThreadId; }

	/*
	* Get the Id of the last thread to pass the stuck thread time.
	* Returns InvalidThreadId if hang detector has not been triggered.
	*/
	uint32 GetLastStuckThreadId() const { return LastStuckThreadId; }

	/*
	* Get delegate for callback on stuck or unstuck thread.
	*/
	FOnThreadStuck& GetOnThreadStuck() { return OnStuck; }
	FOnThreadUnstuck& GetOnThreadUnstuck() { return OnUnstuck; }

	/*
	* Get delegate for callback on hang.
	* Delegate implementation will be called from the hang detector thread and not from the hung thread
	*/
	FOnHangDelegate& GetOnHangDelegate() { return OnHangDelegate; }

	/*
	*  Get hang duration threshold.
	*/
	double GetHangDuration() const { return ConfigHangDuration; };

	//~ Begin FRunnable Interface.
	CORE_API virtual bool Init();
	CORE_API virtual uint32 Run();
	CORE_API virtual void Stop();
	//~ End FRunnable Interface
};

/** Suspends heartbeat measuring for the current thread in the current scope */
struct FSlowHeartBeatScope
{
private:
	bool bSuspendedAllThreads;
	bool bSuspended;
public:
	inline FSlowHeartBeatScope(bool bAllThreads = false)
		: bSuspendedAllThreads(bAllThreads)
		, bSuspended(false)
	{
		if (FThreadHeartBeat* HB = FThreadHeartBeat::GetNoInit())
		{
			bSuspended = true;
			HB->SuspendHeartBeat(bSuspendedAllThreads);
		}
	}
	inline ~FSlowHeartBeatScope()
	{
		if (bSuspended)
		{
			if (FThreadHeartBeat* HB = FThreadHeartBeat::GetNoInit())
			{
				HB->ResumeHeartBeat(bSuspendedAllThreads);
			}
		}
	}
};

/** Simple scope object to put at the top of a function to monitor it completes in a timely fashion */
struct FFunctionHeartBeatScope
{
private:
	bool bStartedMonitor;
public:
	inline FFunctionHeartBeatScope()
		: bStartedMonitor(false)
	{
		if (FThreadHeartBeat* HB = FThreadHeartBeat::GetNoInit())
		{
			bStartedMonitor = true;
			HB->MonitorFunctionStart();
		}
	}
	inline ~FFunctionHeartBeatScope()
	{
		if (bStartedMonitor)
		{
			if (FThreadHeartBeat* HB = FThreadHeartBeat::GetNoInit())
			{
				HB->MonitorFunctionEnd();
			}
		}
	}
};


// When 1, performs a full symbol lookup in hitch call stacks, otherwise only
// a backtrace is performed and the raw addresses are written to the log.
#ifndef LOOKUP_SYMBOLS_IN_HITCH_STACK_WALK
#define LOOKUP_SYMBOLS_IN_HITCH_STACK_WALK 0
#endif

#ifndef WALK_STACK_ON_HITCH_DETECTED
#define WALK_STACK_ON_HITCH_DETECTED 0
#endif

class FGameThreadHitchHeartBeatThreaded : public FRunnable
{
	static CORE_API FGameThreadHitchHeartBeatThreaded* Singleton;

	/** Thread to run the worker FRunnable on */
	FRunnableThread* Thread;
	/** Stops this thread */
	FThreadSafeCounter StopTaskCounter;
	/** Synch object for the heartbeat */
	FCriticalSection HeartBeatCritical;

#if USE_HITCH_DETECTION
	float HangDuration = -1.0f;
	bool bWalkStackOnHitch;
	double FirstStartTime;
	double FrameStartTime;
	int32 SuspendedCount;
	bool bStartSuspended = false;

#if WALK_STACK_ON_HITCH_DETECTED
#if LOOKUP_SYMBOLS_IN_HITCH_STACK_WALK
	static constexpr SIZE_T StackTraceSize = 65535;
	ANSICHAR StackTrace[StackTraceSize];
#else
	static constexpr uint32 MaxStackDepth = 128;
	uint64 StackTrace[MaxStackDepth]; 
#endif // LOOKUP_SYMBOLS_IN_HITCH_STACK_WALK
#endif // WALK_STACK_ON_HITCH_DETECTED
#endif // USE_HITCH_DETECTION

	FThreadHeartBeatClock Clock;

	CORE_API FGameThreadHitchHeartBeatThreaded();
	CORE_API virtual ~FGameThreadHitchHeartBeatThreaded();

	void InitSettingsInternal();

public:

	enum EConstants
	{
		/** Invalid thread Id used by CheckHeartBeat */
		InvalidThreadId = (uint32)-1
	};

	/** Gets the heartbeat singleton */
	static CORE_API FGameThreadHitchHeartBeatThreaded& Get();
	static CORE_API FGameThreadHitchHeartBeatThreaded* GetNoInit();

	/** Update settings from command line or ini */
	CORE_API void InitSettings();

	/**
	* Called at the start of a frame to register the time we are looking to detect a hitch
	*/
	CORE_API void FrameStart(bool bSkipThisFrame = false);

	CORE_API double GetFrameStartTime();
	CORE_API double GetCurrentTime();

	/**
	* Suspend heartbeat hitch detection. Must call ResumeHeartBeat later to resume.
	*/
	CORE_API void SuspendHeartBeat();

	/**
	* Resume heartbeat hitch detection. Call only after first calling SuspendHeartBeat.
	*/
	CORE_API void ResumeHeartBeat();
	
	/**
	* Check if started suspended.
	*/
	CORE_API bool IsStartedSuspended();

	// No-op, used in FUnixSignalGameHitchHeartBeat
	void Restart() {}
	void PostFork() {}

	//~ Begin FRunnable Interface.
	CORE_API virtual bool Init();
	CORE_API virtual uint32 Run();
	CORE_API virtual void Stop();
	//~ End FRunnable Interface
};

#if PLATFORM_UNIX
typedef FUnixSignalGameHitchHeartBeat FGameThreadHitchHeartBeat;
#else
typedef FGameThreadHitchHeartBeatThreaded FGameThreadHitchHeartBeat;
#endif

/** Suspends hitch detection in the current scope */
struct FDisableHitchDetectorScope
{
	UE_FORCEINLINE_HINT FDisableHitchDetectorScope()
	{
		FGameThreadHitchHeartBeat::Get().SuspendHeartBeat();
	}
	UE_FORCEINLINE_HINT ~FDisableHitchDetectorScope()
	{
		FGameThreadHitchHeartBeat::Get().ResumeHeartBeat();
	}
};
