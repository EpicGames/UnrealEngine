// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CoreGlobals.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"

#ifndef PLATFORM_SUPPORTS_ALL_THREAD_BACKTRACES
	#define PLATFORM_SUPPORTS_ALL_THREAD_BACKTRACES (PLATFORM_WINDOWS || PLATFORM_MAC || PLATFORM_UNIX)
#endif

class FRunnableThread;

/**
 * Manages runnables and runnable threads.
 */
class FThreadManager
{
	/** Critical section for ThreadList */
	FCriticalSection ThreadsCritical;

	using FThreads = TMap<uint32, FRunnableThread*, TInlineSetAllocator<256>>;
	/** List of thread objects to be ticked. */
	FThreads Threads;

	/* Helper variable for catching unexpected modification of the thread map/list. */
	bool bIsThreadListDirty = false;

	bool CheckThreadListSafeToContinueIteration();
	void OnThreadListModified();

public:

	FThreadManager();
	~FThreadManager();

	/**
	* Used internally to add a new thread object.
	*
	* @param Thread thread object.
	* @see RemoveThread
	*/
	CORE_API void AddThread(uint32 ThreadId, FRunnableThread* Thread);

	/**
	* Used internally to remove thread object.
	*
	* @param Thread thread object to be removed.
	* @see AddThread
	*/
	CORE_API void RemoveThread(FRunnableThread* Thread);


	/** Specifies the type of thread event. */
	enum class EThreadEventType
	{
		Added,
		Removed
	};

	/**
	  * Used internally to broadcast about a real thread object events.
	  * A thread-safe version, broadcast might be done from any thread.
	  * 
	  * @see EThreadEventType
	  * @see AddThread
	  * @see RemoveThread
	  */
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FThreadEvent, const uint32 /*ThreadId*/, class FRunnableThread* /*Thread*/, EThreadEventType /*EventType*/);
	FThreadEvent& OnThreadEvent();

	/** Get the number of registered threads */
	int32 NumThreads() const { return Threads.Num(); }

	/** Ticks all fake threads and their runnable objects. */
	CORE_API void Tick();

	/** Returns the name of a thread given its TLS id */
	inline static const FString& GetThreadName(uint32 ThreadId)
	{
		static FString GameThreadName("GameThread");
		static FString RenderThreadName("RenderThread");
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (ThreadId == GGameThreadId)
		{
			return GameThreadName;
		}
		else if (ThreadId == GRenderThreadId)
		{
			return RenderThreadName;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return Get().GetThreadNameInternal(ThreadId);
	}

#if PLATFORM_SUPPORTS_ALL_THREAD_BACKTRACES
	struct FThreadStackBackTrace
	{
		static constexpr uint32 ProgramCountersMaxStackSize = 100;
		typedef TArray<uint64, TFixedAllocator<ProgramCountersMaxStackSize>> FProgramCountersArray;

		uint32 ThreadId;
		FString ThreadName;
		FProgramCountersArray ProgramCounters;
	};

	CORE_API void GetAllThreadStackBackTraces(TArray<FThreadStackBackTrace>& StackTraces);

	/*
	 * Enumerates through all thread stack backtraces and calls the provided function for each one.
	 * The callback must return true to continue enumerating, or return false to stop early.
	 *
	 * This function is primarily intended to iterate over stack traces in a crashing context and
	 * avoids allocation of additional memory. It does not, therefore, perform any safety checks to
	 * ensure that the list of threads is not modified mid-iteration or that the callback does not
	 * itself allocate memory.
	 *
	 * Similarly, the thread name and stack trace array memory are only valid for the duration of the
	 * callback's execution and must be copied elsewhere if they are to be used beyond its scope.
	 */
	CORE_API void ForEachThreadStackBackTrace(TFunctionRef<bool(uint32 ThreadId, const TCHAR* ThreadName, const TConstArrayView<uint64>& StackTrace)> Func);
#endif

	/**
	 * Enumerate each thread.
	 *
	 */
	CORE_API void ForEachThread(TFunction<void(uint32 ThreadId, FRunnableThread* Thread)> Func);

	/**
	 * Access to the singleton object.
	 *
	 * @return Thread manager object.
	 */
	static CORE_API FThreadManager& Get();

private:

	friend class FForkProcessHelper;

	/** Returns a list of registered forkable threads  */
	CORE_API TArray<FRunnableThread*> GetForkableThreads();

	/** Returns internal name of a the thread given its TLS id */
	CORE_API const FString& GetThreadNameInternal(uint32 ThreadId);

	/** Notification that the parent is about to fork */
	void HandleOnParentPreFork();

	/** Broadcasts whenever a thread has been added or removed. See EThreadEventType for details */
	FThreadEvent ThreadEvent;
};
