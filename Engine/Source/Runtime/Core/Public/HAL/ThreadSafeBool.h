// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/ThreadSafeCounter.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// DEPRECATED. Please use `std::atomic<bool>`
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Thread safe bool, wraps FThreadSafeCounter
 */
class FThreadSafeBool
	: private FThreadSafeCounter
{
public:
	/**
	 * Constructor optionally takes value to initialize with, otherwise initializes false
	 * @param bValue	Value to initialize with
	 */
	FThreadSafeBool(bool bValue = false)
		: FThreadSafeCounter(bValue ? 1 : 0)
	{}

	/**
	 * Operator to use this struct as a bool with thread safety
	 */
	UE_FORCEINLINE_HINT operator bool() const
	{
		return GetValue() != 0;
	}

	/**
	 * Operator to set the bool value with thread safety
	 */
	inline bool operator=(bool bNewValue)
	{
		Set(bNewValue ? 1 : 0);
		return bNewValue;
	}

	/**
	 * Sets a new value atomically, and returns the old value.
	 *
	 * @param bNewValue   Value to set
	 * @return The old value
	 */
	UE_FORCEINLINE_HINT bool AtomicSet(bool bNewValue)
	{
		return Set(bNewValue ? 1 : 0) != 0;
	}
};
