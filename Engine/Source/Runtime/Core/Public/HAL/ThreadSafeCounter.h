// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformAtomics.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// DEPRECATED. Please use `std::atomic<int32>`.
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Thread safe counter */
class FThreadSafeCounter
{
public:
	using IntegerType = int32;

	/**
	 * Constructor, initializing counter to passed in value.
	 *
	 * @param Value	Value to initialize counter to
	 */
	FThreadSafeCounter(int32 Value = 0)
	{
#if USING_THREAD_SANITISER
		FPlatformAtomics::AtomicStore(&Counter, Value);
#else
		Counter = Value;
#endif
	}

	/**
	 * Copy Constructor.
	 *
	 * If the counter in the Other parameter is changing from other threads, there are no
	 * guarantees as to which values you will get up to the caller to not care, synchronize
	 * or other way to make those guarantees.
	 *
	 * @param Other The other thread safe counter to copy
	 */
	FThreadSafeCounter( const FThreadSafeCounter& Other )
		: FThreadSafeCounter(Other.GetValue())
	{
	}

	/**
	 * Increment and return new value.	
	 *
	 * @return the new, incremented value
	 * @see Add, Decrement, Reset, Set, Subtract
	 */
	int32 Increment()
	{
		return FPlatformAtomics::InterlockedIncrement(&Counter);
	}

	/**
	 * Adds an amount and returns the old value.	
	 *
	 * @param Amount Amount to increase the counter by
	 * @return the old value
	 * @see Decrement, Increment, Reset, Set, Subtract
	 */
	int32 Add( int32 Amount )
	{
		return FPlatformAtomics::InterlockedAdd(&Counter, Amount);
	}

	/**
	 * Decrement and return new value.
	 *
	 * @return the new, decremented value
	 * @see Add, Increment, Reset, Set, Subtract
	 */
	int32 Decrement()
	{
		return FPlatformAtomics::InterlockedDecrement(&Counter);
	}

	/**
	 * Subtracts an amount and returns the old value.	
	 *
	 * @param Amount Amount to decrease the counter by
	 * @return the old value
	 * @see Add, Decrement, Increment, Reset, Set
	 */
	int32 Subtract( int32 Amount )
	{
		return FPlatformAtomics::InterlockedAdd(&Counter, -Amount);
	}

	/**
	 * Sets the counter to a specific value and returns the old value.
	 *
	 * @param Value	Value to set the counter to
	 * @return The old value
	 * @see Add, Decrement, Increment, Reset, Subtract
	 */
	int32 Set( int32 Value )
	{
		return FPlatformAtomics::InterlockedExchange(&Counter, Value);
	}

	/**
	 * Resets the counter's value to zero.
	 *
	 * @return the old value.
	 * @see Add, Decrement, Increment, Set, Subtract
	 */
	int32 Reset()
	{
		return Set(0);
	}

	/**
	 * Gets the current value.
	 *
	 * @return the current value
	 */
	int32 GetValue() const
	{
		return FPlatformAtomics::AtomicRead(&Counter);
	}

private:

	/** Assignment is intentionally disallowed, as usage wouldn't be thread safe. */
	void operator=(const FThreadSafeCounter& Other) = delete;

	/** Thread-safe counter */
	volatile int32 Counter;
};
