// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/ThreadSafeCounter.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// DEPRECATED. Please use `std::atomic<int64>`
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Thread safe counter for 64bit ints */
class FThreadSafeCounter64
{
public:
	typedef int64 IntegerType;

	/**
	* Default constructor.
	*
	* Initializes the counter to 0.
	*/
	FThreadSafeCounter64()
	{
		Counter = 0;
	}

	/**
	* Copy Constructor.
	*
	* If the counter in the Other parameter is changing from other threads, there are no
	* guarantees as to which values you will get up to the caller to not care, synchronize
	* or other way to make those guarantees.
	*
	* @param Other The other thread safe counter to copy
	*/
	FThreadSafeCounter64(const FThreadSafeCounter64& Other)
	{
		Counter = Other.GetValue();
	}

	/** Assignment has the same caveats as the copy ctor. */
	FThreadSafeCounter64& operator=(const FThreadSafeCounter64& Other)
	{
		if (this == &Other) return *this;
		Set(Other.GetValue());
		return *this;
	}


	/**
	* Constructor, initializing counter to passed in value.
	*
	* @param Value	Value to initialize counter to
	*/
	FThreadSafeCounter64(int64 Value)
	{
		Counter = Value;
	}

	/**
	* Increment and return new value.
	*
	* @return the new, incremented value
	* @see Add, Decrement, Reset, Set, Subtract
	*/
	int64 Increment()
	{
		return FPlatformAtomics::InterlockedIncrement(&Counter);
	}

	/**
	* Adds an amount and returns the old value.
	*
	* @param Amount Amount to increase the counter by
	* @return the old value
	* @see Decrement, Increment, Reset, Set, Subtract
	*/
	int64 Add(int64 Amount)
	{
		return FPlatformAtomics::InterlockedAdd(&Counter, Amount);
	}

	/**
	* Decrement and return new value.
	*
	* @return the new, decremented value
	* @see Add, Increment, Reset, Set, Subtract
	*/
	int64 Decrement()
	{
		return FPlatformAtomics::InterlockedDecrement(&Counter);
	}

	/**
	* Subtracts an amount and returns the old value.
	*
	* @param Amount Amount to decrease the counter by
	* @return the old value
	* @see Add, Decrement, Increment, Reset, Set
	*/
	int64 Subtract(int64 Amount)
	{
		return FPlatformAtomics::InterlockedAdd(&Counter, -Amount);
	}

	/**
	* Sets the counter to a specific value and returns the old value.
	*
	* @param Value	Value to set the counter to
	* @return The old value
	* @see Add, Decrement, Increment, Reset, Subtract
	*/
	int64 Set(int64 Value)
	{
		return FPlatformAtomics::InterlockedExchange(&Counter, Value);
	}

	/**
	* Resets the counter's value to zero.
	*
	* @return the old value.
	* @see Add, Decrement, Increment, Set, Subtract
	*/
	int64 Reset()
	{
		return FPlatformAtomics::InterlockedExchange(&Counter, 0);
	}

	/**
	* Gets the current value.
	*
	* @return the current value
	*/
	int64 GetValue() const
	{
		return FPlatformAtomics::AtomicRead(&Counter);
	}

private:
	/** Thread-safe counter */
	volatile int64 Counter;
};
