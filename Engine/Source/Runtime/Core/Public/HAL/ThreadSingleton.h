// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Function.h"
#include "HAL/PlatformTLS.h"
#include "HAL/TlsAutoCleanup.h"

template <typename FuncType> class TFunctionRef;

/**
 * Thread singleton initializer.
 */
class FThreadSingletonInitializer
{
public:

	/**
	* @return an instance of a singleton for the current thread.
	*/
	static CORE_API FTlsAutoCleanup* Get( TFunctionRef<FTlsAutoCleanup*()> CreateInstance, uint32& TlsSlot );

	/**
	 * @return an instance of the singleton if it exists on the current thread.
	 */
	static CORE_API FTlsAutoCleanup* TryGet(uint32& TlsSlot);

	/**
	* @return sets the TLS store to the instance and returns the previous instance.
	*/
	static CORE_API FTlsAutoCleanup* Inject(FTlsAutoCleanup* Instance, uint32& TlsSlot);
};

#define UE_DECLARE_THREAD_SINGLETON_TLS(Type, Api) template<> Api uint32& TThreadSingleton<Type>::GetTlsSlot();
#define UE_DEFINE_THREAD_SINGLETON_TLS(Type, Api) template<> Api uint32& TThreadSingleton<Type>::GetTlsSlot() { static uint32 TlsSlot = FPlatformTLS::InvalidTlsSlot; return TlsSlot; }

/**
 * This a special version of singleton. It means that there is created only one instance for each thread.
 * Calling Get() method is thread-safe.
 */
template < class T >
class TThreadSingleton : public FTlsAutoCleanup
{
#if PLATFORM_UNIX || PLATFORM_APPLE
	/**
	 * @return TLS slot that holds a TThreadSingleton.
	 */
	CORE_API static uint32& GetTlsSlot()
	{
		static uint32 TlsSlot = FPlatformTLS::InvalidTlsSlot;
		return TlsSlot;
	}
#else
	/**
	 * @return TLS slot that holds a TThreadSingleton.
	 */
#if PLATFORM_CONSOLE_DYNAMIC_LINK
	FORCENOINLINE
#endif
	static uint32& GetTlsSlot()
	{
		static uint32 TlsSlot = FPlatformTLS::InvalidTlsSlot;
		return TlsSlot;
	}
#endif

protected:

	/** Default constructor. */
	TThreadSingleton()
		: ThreadId(FPlatformTLS::GetCurrentThreadId())
	{}

	virtual ~TThreadSingleton()
	{
		// Clean the dangling pointer from the TLS.
		check(GetTlsSlot() != FPlatformTLS::InvalidTlsSlot);
		if(((FTlsAutoCleanup*)FPlatformTLS::GetTlsValue(GetTlsSlot())) == static_cast<FTlsAutoCleanup*>(this))
		{
			FPlatformTLS::SetTlsValue(GetTlsSlot(), nullptr);
		}
	}

	/**
	 * @return a new instance of the thread singleton.
	 */
	static FTlsAutoCleanup* CreateInstance()
	{
		return new T();
	}

	/** Thread ID of this thread singleton. */
	const uint32 ThreadId;

public:

	/**
	 *	@return an instance of a singleton for the current thread.
	 */
	FORCEINLINE static T& Get()
	{
		return *(T*)FThreadSingletonInitializer::Get( [](){ return (FTlsAutoCleanup*)new T(); }, T::GetTlsSlot() ); //-V572
	}

	/**
	 *	@param CreateInstance Function to call when a new instance must be created.
	 *	@return an instance of a singleton for the current thread.
	 */
	FORCEINLINE static T& Get(TFunctionRef<FTlsAutoCleanup*()> CreateInstance)
	{
		return *(T*)FThreadSingletonInitializer::Get(CreateInstance, T::GetTlsSlot());
	}

	/**
	 *	@return pointer to an instance of a singleton for the current thread. May be nullptr, prefer to use access by reference
	 */
	FORCEINLINE static T* TryGet()
	{
		return (T*)FThreadSingletonInitializer::TryGet( T::GetTlsSlot() );
	}

	/**
	* @return sets the TLS store to the instance and returns the previous instance.
	*/
	FORCEINLINE static T* Inject(T* Instance)
	{
		return (T*)FThreadSingletonInitializer::Inject(Instance, T::GetTlsSlot());
	}
};
