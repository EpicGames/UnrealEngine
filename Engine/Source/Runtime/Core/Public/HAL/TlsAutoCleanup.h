// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/** 
Base class for objects in TLS that support auto-cleanup. 
Polymorphically deletes registered instances on thread exit.
*/
class FTlsAutoCleanup
{
public:
	/** Virtual destructor. */
	virtual ~FTlsAutoCleanup() = default;

	/** Register this instance to be auto-cleanup. */
	CORE_API void Register();
};

/** Wrapper for values to be stored in TLS that support auto-cleanup. */
template< class T >
class TTlsAutoCleanupValue
	: public FTlsAutoCleanup
{
public:

	/** Constructor. */
	TTlsAutoCleanupValue(const T& InValue)
		: Value(InValue)
	{ }

	/** Constructor. */
	TTlsAutoCleanupValue(T&& InValue)
		: Value(MoveTemp(InValue))
	{ }

	/** Gets the value. */
	T Get() const
	{
		return Value;
	}

	/** Gets the value. */
	T& Get()
	{
		return Value;
	}

	/* Sets the value. */
	void Set(const T& InValue)
	{
		Value = InValue;
	}

	/* Sets the value. */
	void Set(T&& InValue)
	{
		Value = MoveTemp(InValue);
	}

private:

	/** The value. */
	T Value;
};
