// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "HAL/MemoryBase.h"
#include "HAL/PlatformMemory.h"
#include "ProfilingDebugging/MemoryTrace.h"

#include <type_traits>

#ifndef UE_USE_VERYLARGEPAGEALLOCATOR
#define UE_USE_VERYLARGEPAGEALLOCATOR 0
#endif

#ifndef UE_ALLOW_OSMEMORYLOCKFREE
#define UE_ALLOW_OSMEMORYLOCKFREE 0
#endif


// Sizes.

#if STATS
#define MALLOC_GT_HOOKS 1
#else
#define MALLOC_GT_HOOKS 0
#endif

#if MALLOC_GT_HOOKS
CORE_API void DoGamethreadHook(int32 Index);
#else
UE_FORCEINLINE_HINT void DoGamethreadHook(int32 Index)
{ 
}
#endif

#define TIME_MALLOC (0)

#if TIME_MALLOC

struct FScopedMallocTimer
{
	static CORE_API uint64 GTotalCycles[4];
	static CORE_API uint64 GTotalCount[4];
	static CORE_API uint64 GTotalMisses[4];

	int32 Index;
	uint64 Cycles;

	inline FScopedMallocTimer(int32 InIndex)
		: Index(InIndex)
		, Cycles(FPlatformTime::Cycles64())
	{
	}
	inline ~FScopedMallocTimer()
	{
		uint64 Add = uint64(FPlatformTime::Cycles64() - Cycles);
		FPlatformAtomics::InterlockedAdd((volatile int64*)&GTotalCycles[Index], Add);
		uint64 Was = FPlatformAtomics::InterlockedAdd((volatile int64*)&GTotalCount[Index], 1);

		extern CORE_API bool GIsRunning;
		if (GIsRunning && Index == 1 && (Was & 0xffff) == 0)
		{
			Spew();
		}
	}
	static UE_FORCEINLINE_HINT void Miss(int32 InIndex)
	{
		FPlatformAtomics::InterlockedAdd((volatile int64*)&GTotalMisses[InIndex], 1);
	}
	static CORE_API void Spew();
}; 
#else
struct FScopedMallocTimer
{
	UE_FORCEINLINE_HINT FScopedMallocTimer(int32 InIndex)
	{
	}
	UE_FORCEINLINE_HINT ~FScopedMallocTimer() = default;
	UE_FORCEINLINE_HINT void Hit(int32 InIndex)
	{
	}
}; 
#endif


/*-----------------------------------------------------------------------------
	FMemory.
-----------------------------------------------------------------------------*/

struct FMemory
{
	/** Some allocators can be given hints to treat allocations differently depending on how the memory is used, it's lifetime etc. */
	enum AllocationHints
	{
		None = -1,
		Default,
		Temporary,
		SmallPool,

		Max
	};


	/** @name Memory functions (wrapper for FPlatformMemory) */

	static UE_FORCEINLINE_HINT void* Memmove( void* Dest, const void* Src, SIZE_T Count )
	{
		return FPlatformMemory::Memmove( Dest, Src, Count );
	}

	static UE_FORCEINLINE_HINT int32 Memcmp( const void* Buf1, const void* Buf2, SIZE_T Count )
	{
		return FPlatformMemory::Memcmp( Buf1, Buf2, Count );
	}

	static UE_FORCEINLINE_HINT void* Memset(void* Dest, uint8 Char, SIZE_T Count)
	{
		return FPlatformMemory::Memset( Dest, Char, Count );
	}

	template< class T > 
	static inline void Memset( T& Src, uint8 ValueToSet )
	{
		static_assert( !std::is_pointer_v<T>, "For pointers use the three parameters function");
		Memset( &Src, ValueToSet, sizeof( T ) );
	}

	static UE_FORCEINLINE_HINT void* Memzero(void* Dest, SIZE_T Count)
	{
		return FPlatformMemory::Memzero( Dest, Count );
	}

	/** Returns true if memory is zeroes, false otherwise. */
	static inline bool MemIsZero(const void* Ptr, SIZE_T Count)
	{
		// first pass implementation
		uint8* Start = (uint8*)Ptr;
		uint8* End = Start + Count;
		while (Start < End)
		{
			if ((*Start++) != 0)
			{
				return false;
			}
		}

		return true;
	}

	template< class T > 
	static inline void Memzero( T& Src )
	{
		static_assert( !std::is_pointer_v<T>, "For pointers use the two parameters function");
		Memzero( &Src, sizeof( T ) );
	}

	static UE_FORCEINLINE_HINT void* Memcpy(void* Dest, const void* Src, SIZE_T Count)
	{
		return FPlatformMemory::Memcpy(Dest,Src,Count);
	}

	template< class T > 
	static inline void Memcpy( T& Dest, const T& Src )
	{
		static_assert( !std::is_pointer_v<T>, "For pointers use the three parameters function");
		Memcpy( &Dest, &Src, sizeof( T ) );
	}

	static UE_FORCEINLINE_HINT void* BigBlockMemcpy(void* Dest, const void* Src, SIZE_T Count)
	{
		return FPlatformMemory::BigBlockMemcpy(Dest,Src,Count);
	}

	static UE_FORCEINLINE_HINT void* StreamingMemcpy(void* Dest, const void* Src, SIZE_T Count)
	{
		return FPlatformMemory::StreamingMemcpy(Dest,Src,Count);
	}

	static UE_FORCEINLINE_HINT void* ParallelMemcpy(void* Dest, const void* Src, SIZE_T Count, EMemcpyCachePolicy Policy = EMemcpyCachePolicy::StoreCached)
	{
		return FPlatformMemory::ParallelMemcpy(Dest, Src, Count, Policy);
	}

	static UE_FORCEINLINE_HINT void Memswap( void* Ptr1, void* Ptr2, SIZE_T Size )
	{
		FPlatformMemory::Memswap(Ptr1,Ptr2,Size);
	}

	//
	// C style memory allocation stubs that fall back to C runtime
	//
	UE_ALLOCATION_FUNCTION(1) static inline void* SystemMalloc(SIZE_T Size)
	{
		void* Ptr = ::malloc(Size);
		MemoryTrace_Alloc(uint64(Ptr), Size, 0, EMemoryTraceRootHeap::SystemMemory);
		return Ptr;
	}

	static inline void SystemFree(void* Ptr)
	{
		MemoryTrace_Free(uint64(Ptr), EMemoryTraceRootHeap::SystemMemory);
		::free(Ptr);
	}

	//
	// C style memory allocation stubs.
	//

	UE_ALLOCATION_FUNCTION(1, 2) static FORCENOINLINE CORE_API void* Malloc(SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT);
	UE_ALLOCATION_FUNCTION(2, 3) static FORCENOINLINE CORE_API void* Realloc(void* Original, SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT);
	static FORCENOINLINE CORE_API void Free(void* Original);
	static FORCENOINLINE CORE_API SIZE_T GetAllocSize(void* Original);

	UE_ALLOCATION_FUNCTION(1, 2) static FORCENOINLINE CORE_API void* MallocZeroed(SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT);

	/**
	* For some allocators this will return the actual size that should be requested to eliminate
	* internal fragmentation. The return value will always be >= Count. This can be used to grow
	* and shrink containers to optimal sizes.
	* This call is always fast and threadsafe with no locking.
	*/
	static FORCENOINLINE CORE_API SIZE_T QuantizeSize(SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT);

	/**
	* Releases as much memory as possible. Must be called from the main thread.
	*/
	static CORE_API void Trim(bool bTrimThreadCaches = true);

	/**
	* Set up TLS caches on the current thread. These are the threads that we can trim.
	*/
	static CORE_API void SetupTLSCachesOnCurrentThread();

	/**
	* Clears the TLS caches on the current thread and disables any future caching.
	*/
	static CORE_API void ClearAndDisableTLSCachesOnCurrentThread();

	/**
	* Mark TLS caches for the current thread as used. Thread has woken up to do some processing and needs its TLS caches back.
	*/
	static CORE_API void MarkTLSCachesAsUsedOnCurrentThread();

	/**
	* Mark TLS caches for current thread as unused. Typically before going to sleep. These are the threads that we can trim without waking them up.
	*/
	static CORE_API void MarkTLSCachesAsUnusedOnCurrentThread();

	/**
	 * A helper function that will perform a series of random heap allocations to test
	 * the internal validity of the heap. Note, this function will "leak" memory, but another call
	 * will clean up previously allocated blocks before returning. This will help to A/B testing
	 * where you call it in a good state, do something to corrupt memory, then call this again
	 * and hopefully freeing some pointers will trigger a crash.
	 */
	static CORE_API void TestMemory();
	/**
	* Called once main is started and we have -purgatorymallocproxy.
	* This uses the purgatory malloc proxy to check if things are writing to stale pointers.
	*/
	static CORE_API void EnablePurgatoryTests();
	/**
	* Called once main is started and we have -poisonmallocproxy.
	*/
	static CORE_API void EnablePoisonTests();
	/**
	* Set global allocator instead of creating it lazily on first allocation.
	* Must only be called once and only if lazy init is disabled via a macro.
	*/
	static CORE_API void ExplicitInit(FMalloc& Allocator);

	// These versions are called either at startup or in the event of a crash
	static CORE_API void* MallocExternal(SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT);
	static CORE_API void* ReallocExternal(void* Original, SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT);
	static CORE_API void FreeExternal(void* Original);
	static CORE_API SIZE_T GetAllocSizeExternal(void* Original);
	static CORE_API void* MallocZeroedExternal(SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT);
	static CORE_API SIZE_T QuantizeSizeExternal(SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT);

private:
	static CORE_API void GCreateMalloc();
};
