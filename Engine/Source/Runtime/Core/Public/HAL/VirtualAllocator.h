// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Async/UniqueLock.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/PlatformMutex.h"
#include "HAL/PlatformProcess.h"
#include "Templates/AlignmentTemplates.h"

class FVirtualAllocator
{
	struct FFreeLink
	{
		void *Ptr = nullptr;
		FFreeLink* Next = nullptr;
	};

	struct FPerBlockSize
	{
		int64 AllocBlocksSize = 0;
		int64 FreeBlocksSize = 0;
		FFreeLink* FirstFree = nullptr;
	};

	UE::FPlatformRecursiveMutex Mutex;

	uint8* LowAddress;
	uint8* HighAddress;

	size_t TotalSize;
	size_t PageSize;
	size_t MaximumAlignment;

	uint8* NextAlloc;

	FFreeLink* RecycledLinks;
	int64 LinkSize;

	bool bBacksMalloc;

	FPerBlockSize Blocks[64]; 

	void FreeVirtualByBlock(void* Ptr, FPerBlockSize& Block, size_t AlignedSize)
	{
		// already locked
		if (!RecycledLinks)
		{
			void *Alloc;
			// If we ARE malloc, then we know we can (and must) commit part of our range for free links. Otherwise, we can just use malloc (and can't use our VM space anyway)
			if (bBacksMalloc)
			{
				Alloc = AllocNewVM(MaximumAlignment);
				uint32 Pages = MaximumAlignment / FPlatformMemory::FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment();
				check(Pages && MaximumAlignment % FPlatformMemory::FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment() == 0);
				FPlatformMemory::FPlatformVirtualMemoryBlock VMBlock(Alloc, Pages);
				VMBlock.Commit();
			}
			else
			{
				Alloc = FMemory::Malloc(MaximumAlignment);
			}
			for (int32 Index = 0; (Index + 1) * sizeof(FFreeLink) <= MaximumAlignment; Index++)
			{
				FFreeLink* NewLink = new ((void*)(((uint8*)Alloc + Index * sizeof(FFreeLink)))) FFreeLink;
				NewLink->Next = RecycledLinks;
				RecycledLinks = NewLink;
			}
			LinkSize += MaximumAlignment;
			checkSlow(RecycledLinks); // To suppress a static analysis warning about RecycledLinks being null below.
		}

		FFreeLink* Link = RecycledLinks;
		RecycledLinks = Link->Next;
		check(!Link->Ptr && Ptr);
		Link->Ptr = Ptr;
		Link->Next = Block.FirstFree;
		Block.FirstFree = Link;

		Block.FreeBlocksSize += AlignedSize;
	}
protected:
	size_t SpaceConsumed;
	size_t SpaceConsumedPeak;

	virtual uint8* AllocNewVM(size_t AlignedSize)
	{
		uint8* Result = NextAlloc;
		check(IsAligned(Result, MaximumAlignment) && IsAligned(AlignedSize, MaximumAlignment));
		NextAlloc = Result + AlignedSize;
		SpaceConsumed = NextAlloc - LowAddress;
		if (SpaceConsumed > SpaceConsumedPeak)
		{
			SpaceConsumedPeak = SpaceConsumed;
		}
		return Result;
	}

public:

	FVirtualAllocator(void *InLowAdress, void* InHighAddress, size_t InPageSize, size_t InMaximumAlignment, bool bInBacksMalloc)
		: LowAddress((uint8*)InLowAdress)
		, HighAddress((uint8*)InHighAddress)
		, PageSize(InPageSize)
		, MaximumAlignment(FMath::Max(InMaximumAlignment, InPageSize))
		, NextAlloc((uint8*)InLowAdress)
		, RecycledLinks(nullptr)
		, LinkSize(0)
		, bBacksMalloc(bInBacksMalloc)
		, SpaceConsumed(0)
		, SpaceConsumedPeak(0)
	{
		TotalSize = HighAddress - LowAddress;
		check(LowAddress && HighAddress && LowAddress < HighAddress && IsAligned(LowAddress, MaximumAlignment));
	}

	virtual ~FVirtualAllocator() = default;

	uint32 GetPagesForSizeAndAlignment(size_t Size, size_t Alignment = 1) const
	{
		check(Alignment <= MaximumAlignment && Alignment > 0);
		size_t SizeAndAlignment = FMath::Max(Align(Size, Alignment), PageSize);
		if (SizeAndAlignment * 2 >= TotalSize)
		{
			// this is hack, MB3 will ask for tons of virtual and never free it, so we won't round up to power of two
			size_t Pages = SizeAndAlignment / PageSize;
			check(Pages == uint32(Pages)); // overflow of uint32
			return uint32(Pages);
		}
		size_t BlockIndex = FMath::CeilLogTwo64(FMath::Max(SizeAndAlignment, PageSize));
		size_t AlignedSize = size_t(1) << BlockIndex;
		check(AlignedSize % PageSize == 0);
		size_t Pages = AlignedSize / PageSize;
		check(Pages == uint32(Pages)); // overflow of uint32
		return uint32(Pages);
	}

	void* AllocateVirtualPages(uint32 NumPages, size_t AlignmentForCheck = 1) 
	{
		check(AlignmentForCheck <= MaximumAlignment && AlignmentForCheck > 0 && NumPages);


		size_t BlockIndex = FMath::CeilLogTwo64(NumPages * PageSize);
		size_t AlignedSize = size_t(1) << BlockIndex;
		bool bHackForHugeBlock = false;
		if (size_t(NumPages) * PageSize * 2 >= TotalSize)
		{
			// this is hack, MB3 will ask for tons of virtual and never free it, so we won't round up to power of two
			AlignedSize = size_t(NumPages) * PageSize;
			bHackForHugeBlock = true;
		}
		FPerBlockSize& Block = Blocks[BlockIndex];

		UE::TUniqueLock Lock(Mutex);


		uint8* Result;
		if (Block.FirstFree)
		{
			check(!bHackForHugeBlock);
			Result = (uint8*)Block.FirstFree->Ptr;
			check(Result);
			Block.FirstFree->Ptr = nullptr;

			FFreeLink* Next = RecycledLinks;
			RecycledLinks = Block.FirstFree;
			Block.FirstFree = Block.FirstFree->Next;
			check(!Block.FirstFree || Block.FirstFree->Ptr);
			RecycledLinks->Next = Next;

			check(IsAligned(Result, FMath::Min(AlignedSize, MaximumAlignment)));
			Block.FreeBlocksSize -= AlignedSize;
		}
		else
		{
			size_t AllocSize = FMath::Max(AlignedSize, MaximumAlignment);
			uint8* LocalNextAlloc = AllocNewVM(AllocSize);
			check(IsAligned(LocalNextAlloc, MaximumAlignment));

			uint8* NewNextAlloc = LocalNextAlloc + AllocSize;

			if (NewNextAlloc > HighAddress)
			{
				FPlatformMemory::OnOutOfMemory(HighAddress - LowAddress, 0);
			}

			Block.AllocBlocksSize += AllocSize;
			Result = LocalNextAlloc;
			check(Result);
			LocalNextAlloc += AlignedSize;
			while (LocalNextAlloc < NewNextAlloc && !bHackForHugeBlock)
			{
				FreeVirtualByBlock(LocalNextAlloc, Block, AlignedSize);
				LocalNextAlloc += AlignedSize;
			}
		}
		check(IsAligned(Result, AlignmentForCheck));
		return Result;
	}
	
	void FreeVirtual(void* Ptr, uint32 NumPages)
	{
		if (size_t(NumPages) * PageSize * 2 >= TotalSize)
		{
			// this is hack, MB3 will ask for tons of virtual and never free it, so we won't round up to power of two
			check(!"Huge vm blocks may not be freed.");
			return;
		}
		size_t BlockIndex = FMath::CeilLogTwo64(NumPages * PageSize);
		size_t AlignedSize = size_t(1) << BlockIndex;

		FPerBlockSize& Block = Blocks[BlockIndex];
		UE::TUniqueLock Lock(Mutex);
		FreeVirtualByBlock(Ptr, Block, AlignedSize);
	}

	struct FVirtualAllocatorStatsPerBlockSize
	{
		size_t AllocBlocksSize;
		size_t FreeBlocksSize;
	};
	struct FVirtualAllocatorStats
	{
		size_t PageSize;
		size_t MaximumAlignment;
		size_t VMSpaceTotal;
		size_t VMSpaceConsumed;
		size_t VMSpaceConsumedPeak;

		size_t FreeListLinks;

		FVirtualAllocatorStatsPerBlockSize BlockStats[64];
	};

	void GetStats(FVirtualAllocatorStats& OutStats)
	{
		UE::TUniqueLock Lock(Mutex);
		OutStats.PageSize = PageSize;
		OutStats.MaximumAlignment = MaximumAlignment;
		OutStats.VMSpaceTotal = HighAddress - LowAddress;
		OutStats.VMSpaceConsumed = SpaceConsumed;
		OutStats.VMSpaceConsumedPeak = SpaceConsumedPeak;
		OutStats.FreeListLinks = LinkSize;

		for (int32 Index = 0; Index < 64; Index++)
		{
			OutStats.BlockStats[Index].AllocBlocksSize = Blocks[Index].AllocBlocksSize;
			OutStats.BlockStats[Index].FreeBlocksSize = Blocks[Index].FreeBlocksSize;
		}
	}
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#	include "HAL/CriticalSection.h"
#	include "Misc/ScopeLock.h"
#endif
