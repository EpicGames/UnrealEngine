// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "HAL/UnrealMemory.h"
#include "Memory/MemoryFwd.h"
#include "Memory/MemoryView.h"
#include "Misc/AssertionMacros.h"
#include "String/BytesToHex.h"
#include "String/HexToBytes.h"
#include "Templates/TypeCompatibleBytes.h"

class FArchive;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Stores a BLAKE3 hash. */
struct alignas(uint32) FBlake3Hash
{
public:
	using ByteArray = uint8[32];

	/** Construct a zero hash. */
	FBlake3Hash() = default;

	/** Construct a hash from an array of 32 bytes. */
	inline explicit FBlake3Hash(const ByteArray& Hash);

	/** Construct a hash from a 64-character hex string. */
	inline explicit FBlake3Hash(FAnsiStringView HexHash);
	inline explicit FBlake3Hash(FWideStringView HexHash);
	inline explicit FBlake3Hash(FUtf8StringView HexHash);

	/** Construct a hash from a view of 32 bytes. */
	inline static FBlake3Hash FromView(FMemoryView Hash);

	/** Reset this to a zero hash. */
	inline void Reset() { *this = FBlake3Hash(); }

	/** Returns whether this is a zero hash. */
	inline bool IsZero() const;

	/** Returns a reference to the raw byte array for the hash. */
	inline ByteArray& GetBytes() { return Hash; }
	inline const ByteArray& GetBytes() const { return Hash; }

	/** A zero hash. */
	static const FBlake3Hash Zero;

	inline bool operator==(const FBlake3Hash& B) const
	{
		return FMemory::Memcmp(GetBytes(), B.GetBytes(), sizeof(decltype(GetBytes()))) == 0;
	}

	inline bool operator!=(const FBlake3Hash& B) const
	{
		return FMemory::Memcmp(GetBytes(), B.GetBytes(), sizeof(decltype(GetBytes()))) != 0;
	}

	inline bool operator<(const FBlake3Hash& B) const
	{
		return FMemory::Memcmp(GetBytes(), B.GetBytes(), sizeof(decltype(GetBytes()))) < 0;
	}

	friend CORE_API FArchive& operator<<(FArchive& Ar, FBlake3Hash& Value);

	friend inline uint32 GetTypeHash(const FBlake3Hash& Value)
	{
		return *reinterpret_cast<const uint32*>(Value.GetBytes());
	}

private:
	ByteArray Hash{};
};

inline const FBlake3Hash FBlake3Hash::Zero;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Calculates a BLAKE3 hash. */
class FBlake3
{
public:
	inline FBlake3() { Reset(); }

	FBlake3(const FBlake3&) = delete;
	FBlake3& operator=(const FBlake3&) = delete;

	/** Reset to the default state in which no input has been written. */
	CORE_API void Reset();

	/** Add the buffer as input to the hash. May be called any number of times. */
	CORE_API void Update(FMemoryView View);
	CORE_API void Update(const void* Data, uint64 Size);
	CORE_API void Update(const FCompositeBuffer& Buffer);

	/**
	 * Finalize the hash of the input data.
	 *
	 * May be called any number of times, and more input may be added after.
	 */
	[[nodiscard]] CORE_API FBlake3Hash Finalize() const;

	/** Calculate the hash of the buffer. */
	[[nodiscard]] CORE_API static FBlake3Hash HashBuffer(FMemoryView View);
	[[nodiscard]] CORE_API static FBlake3Hash HashBuffer(const void* Data, uint64 Size);
	[[nodiscard]] CORE_API static FBlake3Hash HashBuffer(const FCompositeBuffer& Buffer);

private:
	TAlignedBytes<1912, 8> HasherBytes;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

inline FBlake3Hash::FBlake3Hash(const ByteArray& InHash)
{
	FMemory::Memcpy(Hash, InHash, sizeof(ByteArray));
}

inline FBlake3Hash::FBlake3Hash(const FAnsiStringView HexHash)
{
	check(HexHash.Len() == sizeof(ByteArray) * 2);
	UE::String::HexToBytes(HexHash, Hash);
}

inline FBlake3Hash::FBlake3Hash(const FWideStringView HexHash)
{
	check(HexHash.Len() == sizeof(ByteArray) * 2);
	UE::String::HexToBytes(HexHash, Hash);
}

inline FBlake3Hash::FBlake3Hash(const FUtf8StringView HexHash)
{
	check(HexHash.Len() == sizeof(ByteArray) * 2);
	UE::String::HexToBytes(HexHash, Hash);
}

inline FBlake3Hash FBlake3Hash::FromView(const FMemoryView InHash)
{
	checkf(InHash.GetSize() == sizeof(ByteArray),
		TEXT("FBlake3Hash cannot be constructed from a view of %" UINT64_FMT " bytes."), InHash.GetSize());
	FBlake3Hash NewHash;
	FMemory::Memcpy(NewHash.Hash, InHash.GetData(), sizeof(ByteArray));
	return NewHash;
}

inline bool FBlake3Hash::IsZero() const
{
	using UInt32Array = uint32[8];
	static_assert(sizeof(UInt32Array) == sizeof(ByteArray), "Invalid size for UInt32Array");
	for (uint32 Value : reinterpret_cast<const UInt32Array&>(Hash))
	{
		if (Value != 0)
		{
			return false;
		}
	}
	return true;
}

template <typename CharType>
inline TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const FBlake3Hash& Hash)
{
	UE::String::BytesToHexLower(Hash.GetBytes(), Builder);
	return Builder;
}

/** Construct a hash from a 64-character hex string. */
inline void LexFromString(FBlake3Hash& OutHash, const TCHAR* Buffer)
{
	OutHash = FBlake3Hash(Buffer);
}

/** Construct a hash from a 64-character hex string. */
template <typename CharType>
inline bool TryLexFromString(FBlake3Hash& OutHash, TStringView<CharType> String)
{
	if (String.Len() == sizeof(FBlake3Hash) * 2)
	{
		OutHash = FBlake3Hash(String);
		return true;
	}
	return false;
}

/** Convert a hash to a 64-character hex string. */
[[nodiscard]] CORE_API FString LexToString(const FBlake3Hash& Hash);

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Containers/UnrealString.h"
#include "Serialization/Archive.h"
#endif
