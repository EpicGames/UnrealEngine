// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Memory/MemoryView.h"

/**
 * Rolling hash function. Allows efficient computation of the hash of a window into a block of data, which can be moved
 * through the data byte by byte.
 */
struct FBuzHash
{
public:
	/** Reset the state of the hasher. */
	UE_FORCEINLINE_HINT void Reset() { *this = FBuzHash(); }

	/** Gets the current hash value. */
	UE_FORCEINLINE_HINT uint32 Get() const { return State; }

	/** Gets the size of the window. */
	UE_FORCEINLINE_HINT uint64 GetWindowSize() const { return Count; }

	/** Appends a byte to the start of the window and updates the hash. */
	inline void Add(uint8 X)
	{
		State = Rol32(State, 1) ^ Table[X];
		Count++;
	}

	/** Adds an array of bytes to the start of the window. */
	UE_FORCEINLINE_HINT void Add(FMemoryView View)
	{
		Add((const uint8*)View.GetData(), View.GetSize());
	}

	/** Adds an array of bytes to the start of the window. */
	inline void Add(const uint8* Data, uint64 Size)
	{
		for (uint64 Idx = 0; Idx < Size; ++Idx)
		{
			Add(Data[Idx]);
		}
	}

	/** Removes a byte from the back of the sliding window and updates the hash. */
	inline void Sub(uint8 X)
	{
		State = State ^ Rol32(Table[X], uint32(Count - 1));
		Count--;
	}

private:
	CORE_API static const uint32 Table[256];

	uint64 Count = 0;
	uint32 State = 0;

	static inline uint32 Rol32(uint32 V, uint32 N)
	{
		N &= 31;
		return ((V) << (N)) | ((V) >> (32 - N));
	}
};