// Copyright (c) 2011 Google, Inc.
//
// Permission is hereby granted, free of charge, to any person obtaining a copy
// of this software and associated documentation files (the "Software"), to deal
// in the Software without restriction, including without limitation the rights
// to use, copy, modify, merge, publish, distribute, sublicense, and/or sell
// copies of the Software, and to permit persons to whom the Software is
// furnished to do so, subject to the following conditions:
//
// The above copyright notice and this permission notice shall be included in
// all copies or substantial portions of the Software.
//
// THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND, EXPRESS OR
// IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF MERCHANTABILITY,
// FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT. IN NO EVENT SHALL THE
// AUTHORS OR COPYRIGHT HOLDERS BE LIABLE FOR ANY CLAIM, DAMAGES OR OTHER
// LIABILITY, WHETHER IN AN ACTION OF CONTRACT, TORT OR OTHERWISE, ARISING FROM,
// OUT OF OR IN CONNECTION WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN
// THE SOFTWARE.
//
// CityHash, by Geoff Pike and Jyrki Alakuijala
//
// http://code.google.com/p/cityhash/
//
// This file provides a few functions for hashing strings.  All of them are
// high-quality functions in the sense that they pass standard tests such
// as Austin Appleby's SMHasher.  They are also fast.
//
// For 64-bit x86 code, on short strings, we don't know of anything faster than
// CityHash64 that is of comparable quality.  We believe our nearest competitor
// is Murmur3.  For 64-bit x86 code, CityHash64 is an excellent choice for hash
// tables and most other hashing (excluding cryptography).
//
// For 64-bit x86 code, on long strings, the picture is more complicated.
// On many recent Intel CPUs, such as Nehalem, Westmere, Sandy Bridge, etc.,
// CityHashCrc128 appears to be faster than all competitors of comparable
// quality.  CityHash128 is also good but not quite as fast.  We believe our
// nearest competitor is Bob Jenkins' Spooky.  We don't have great data for
// other 64-bit CPUs, but for long strings we know that Spooky is slightly
// faster than CityHash on some relatively recent AMD x86-64 CPUs, for example.
// Note that CityHashCrc128 is declared in citycrc.h.
//
// For 32-bit x86 code, we don't know of anything faster than CityHash32 that
// is of comparable quality.  We believe our nearest competitor is Murmur3A.
// (On 64-bit CPUs, it is typically faster to use the other CityHash variants.)
//
// Functions in the CityHash family are not suitable for cryptography.
//
// Please see CityHash's README file for more details on our performance
// measurements and so on.
//
// WARNING: This code has been only lightly tested on big-endian platforms!
// It is known to work well on little-endian platforms that have a small penalty
// for unaligned reads, such as current Intel and AMD moderate-to-high-end CPUs.
// It should work on all 32-bit and 64-bit platforms that allow unaligned reads;
// bug reports are welcome.
//
// By the way, for some hash functions, given strings a and b, the hash
// of a+b is easily derived from the hashes of a and b.  This property
// doesn't hold for any hash functions in this file.

#ifndef CITY_HASH_H_
#define CITY_HASH_H_

#include "CoreTypes.h"

struct Uint128_64
{
	inline Uint128_64(uint64 InLo, uint64 InHi) : lo(InLo), hi(InHi) {}
	uint64 lo;
	uint64 hi;
};

// Hash function for a byte array.
// NOTE: provided for compatibility, newer code should prefer XXHash.
CORE_API uint64 CityHash64(const char *buf, uint32 len);

// Hash function for a byte array.  For convenience, a 64-bit seed is also
// hashed into the result.
// NOTE: provided for compatibility, newer code should prefer XXHash.
CORE_API uint64 CityHash64WithSeed(const char *buf, uint32 len, uint64 seed);

// Hash function for a byte array.  For convenience, two seeds are also
// hashed into the result.
// NOTE: provided for compatibility, newer code should prefer XXHash.
CORE_API uint64 CityHash64WithSeeds(const char *buf, uint32 len,
	uint64 seed0, uint64 seed1);

// Hash function for a byte array.  Most useful in 32-bit binaries.
// NOTE: provided for compatibility, newer code should prefer XXHash.
CORE_API uint32 CityHash32(const char *buf, uint32 len);

// Hash 128 input bits down to 64 bits of output.
// This is intended to be a reasonably good hash function.
inline uint64 CityHash128to64(const Uint128_64& x) {
	// Murmur-inspired hashing.
	const uint64 kMul = 0x9ddfea08eb382d69ULL;
	uint64 a = (x.lo ^ x.hi) * kMul;
	a ^= (a >> 47);
	uint64 b = (x.hi ^ a) * kMul;
	b ^= (b >> 47);
	b *= kMul;
	return b;
}

#endif  // CITY_HASH_H_
