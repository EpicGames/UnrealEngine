// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CityHash.h"

template<typename T>
uint64 AppendCityHash(const T& Value, uint64 Seed)
{
	return CityHash64WithSeed((const char*)&Value, sizeof(Value), Seed);
}
