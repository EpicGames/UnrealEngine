// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/StringView.h"
#include "CoreTypes.h"
#include "HAL/Platform.h"

namespace UE
{
	/**
	 * Implements a constexpr usable version of FNV1a for strings.
	 * 
	 * Note: This implementation is 'stable' such that the same string of different character widths will
	 * hash to the same value (eg. HashStringFNV1a("Hello") == HashStringFNV1a(L"Hello")).
	 * This behavior is appropriate here as we're operating on strings specifically rather than raw buffers
	 * and we want consistent hashes between different platforms.
	 */
	template<typename HashType, typename CharType>
	constexpr HashType HashStringFNV1a(TStringView<CharType> String)
	{
		static_assert(std::is_same_v<HashType, uint32> || std::is_same_v<HashType, uint64>, "HashType must be an unsigned 32 or 64 bit integer.");
		static_assert(sizeof(CharType) <= 4, "FNV1a only works with characters up to 32 bits.");

		// See https://en.wikipedia.org/wiki/Fowler%E2%80%93Noll%E2%80%93Vo_hash_function#FNV_hash_parameters
		constexpr HashType Offset = []() -> HashType
		{
			if constexpr (sizeof(HashType) == 4)
			{
				return 0x811c9dc5;
			}
			else
			{
				return 0xcbf29ce484222325;
			}
		}();
		constexpr HashType Prime = []() -> HashType
		{
			if constexpr (sizeof(HashType) == 4)
			{
				return 0x01000193;
			}
			else
			{
				return 0x100000001b3;
			}
		}();

		HashType Fnv = Offset;
		for (CharType Char : String)
		{
			// Operate on every character as if it's 4 bytes.
			// Characters < 4 bytes will be padded out with zeros.
			const uint32 Ch = static_cast<uint32>(Char);

			Fnv ^= (Ch >> 24) & 0xff;
			Fnv *= Prime;
			Fnv ^= (Ch >> 16) & 0xff;
			Fnv *= Prime;
			Fnv ^= (Ch >> 8) & 0xff;
			Fnv *= Prime;
			Fnv ^= (Ch) & 0xff;
			Fnv *= Prime;
		}

		return Fnv;
	}

	/* Version taking a string literal. Use this version to force hashing at compile time. */
	template<typename HashType, typename CharType, int32 Len>
	UE_CONSTEVAL HashType HashStringFNV1a(const CharType(&StringLiteral)[Len])
	{
		return HashStringFNV1a<HashType>(MakeStringView(StringLiteral, Len - 1));
	}

	/* Creates a 32-bit FNV1a hash for the given string. */
	template<typename CharType>
	constexpr uint32 HashStringFNV1a32(TStringView<CharType> String)
	{
		return HashStringFNV1a<uint32>(String);
	}

	template<typename CharType, int32 Len>
	UE_CONSTEVAL uint32 HashStringFNV1a32(const CharType(&StringLiteral)[Len])
	{
		return HashStringFNV1a<uint32>(StringLiteral);
	}

	/* Creates a 64-bit FNV1a hash for the given string. */
	template<typename CharType>
	constexpr uint64 HashStringFNV1a64(TStringView<CharType> String)
	{
		return HashStringFNV1a<uint64>(String);
	}

	template<typename CharType, int32 Len>
	UE_CONSTEVAL uint64 HashStringFNV1a64(const CharType(&StringLiteral)[Len])
	{
		return HashStringFNV1a<uint64>(StringLiteral);
	}

} // namespace UE