// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Hash/xxhash.h"
#include "Serialization/MemoryLayout.h"

/* 64-bit xxhash is a good candidate for our use case - relatively compact and well distributed, likelihood of collisions is vanishingly small 
* for the number of inputs we expect (key note that collisions across different shader formats don't matter; we only need to be concerned about the
* maximum number of shaders for any single platform). */
using FHashType = FXxHash64;

/**
 * Hash type used for identifying compiled shaders and shader maps in shader libraries.
 * We inherit from FHashType rather than just directly aliasing so the type is forward-declarable.
 */
struct FShaderHash : public FHashType
{
	FShaderHash() : FHashType() {}
	FShaderHash(const FHashType& Other)
	{
		*this = Other;
	}

	inline FShaderHash& operator=(const FHashType& Other)
	{
		reinterpret_cast<FHashType&>(*this) = Other;
		return *this;
	}
};

ALIAS_TYPE_LAYOUT(FShaderHash, FHashType);

/** Builder for incrementally computing FShaderHash values. */
using FShaderHashBuilder = FXxHash64Builder;
