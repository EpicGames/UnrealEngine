// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Memory/MemoryFwd.h"
#include "Misc/ByteSwap.h"
#include "Serialization/Archive.h"
#include "String/BytesToHex.h"

class FCompositeBuffer;
class FCbFieldView;
class FCbWriter;
template <typename CharType> class TStringBuilderBase;

/** A 64-bit hash from XXH3. */
struct FXxHash64
{
	/**
	 * The hash in its native representation.
	 *
	 * Use the canonical representation from ToByteArray to serialize or display the hash.
	 */
	uint64 Hash{};

	[[nodiscard]] CORE_API static FXxHash64 HashBuffer(FMemoryView View);
	[[nodiscard]] CORE_API static FXxHash64 HashBuffer(const void* Data, uint64 Size);
	[[nodiscard]] CORE_API static FXxHash64 HashBuffer(const FCompositeBuffer& Buffer);

	/**
	 * Hash the buffer in parallel in independent chunks, and hash those hashes.
	 *
	 * Use a ChunkSize large enough to cover task overhead, e.g., 256+ KiB.
	 * Hashing the same buffer with different chunk sizes produces a different output hash.
	 */
	[[nodiscard]] CORE_API static FXxHash64 HashBufferChunked(FMemoryView View, uint64 ChunkSize);
	[[nodiscard]] CORE_API static FXxHash64 HashBufferChunked(const void* Data, uint64 Size, uint64 ChunkSize);

	/** Load the hash from its canonical (big-endian) representation. */
	[[nodiscard]] static inline FXxHash64 FromByteArray(const uint8(&Bytes)[sizeof(uint64)])
	{
		uint64 HashBigEndian;
		FMemory::Memcpy(&HashBigEndian, Bytes, sizeof(uint64));
		return {NETWORK_ORDER64(HashBigEndian)};
	}

	/** Store the hash to its canonical (big-endian) representation. */
	inline void ToByteArray(uint8 (&Bytes)[sizeof(uint64)]) const
	{
		const uint64 HashBigEndian = NETWORK_ORDER64(Hash);
		FMemory::Memcpy(Bytes, &HashBigEndian, sizeof(uint64));
	}

	inline bool operator==(const FXxHash64& B) const
	{
		return Hash == B.Hash;
	}

	inline bool operator!=(const FXxHash64& B) const
	{
		return Hash != B.Hash;
	}

	inline bool operator<(const FXxHash64& B) const
	{
		return Hash < B.Hash;
	}

	inline bool IsZero() const
	{
		return Hash == 0;
	}

	static constexpr int32 GetStringLen() { return sizeof(uint64) * 2; }

	/** Convert the hash to a 16-character hex string. */
	[[nodiscard]] CORE_API FString ToString() const;

	/** Append the 16-character hex representation to the given string. */
	CORE_API void AppendString(FString& Out) const;

	/** Parse the hash from a 16-character hex string. */
	CORE_API void FromString(FStringView Src);

	friend inline uint32 GetTypeHash(const FXxHash64& InHash)
	{
		return uint32(InHash.Hash);
	}

	friend inline FArchive& operator<<(FArchive& Ar, FXxHash64& InHash)
	{
		return Ar << InHash.Hash;
	}

	template <typename CharType>
	friend inline TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const FXxHash64& InHash)
	{
		uint8 Bytes[8];
		InHash.ToByteArray(Bytes);
		UE::String::BytesToHexLower(MakeArrayView(Bytes), Builder);
		return Builder;
	}

	CORE_API friend FString LexToString(const FXxHash64& InHash); 
	CORE_API friend void LexFromString(FXxHash64& InHash, const TCHAR* InString);

	friend FCbWriter& operator<<(FCbWriter& Writer, const FXxHash64& InHash) { return InHash.WriteCompactBinary(Writer); }
	friend bool LoadFromCompactBinary(const FCbFieldView& Field, FXxHash64& OutHash) { return OutHash.LoadFromCompactBinary(Field); }

	CORE_API bool LoadFromCompactBinary(const FCbFieldView& Field);
	CORE_API FCbWriter& WriteCompactBinary(FCbWriter& Writer) const;
};

DECLARE_INTRINSIC_TYPE_LAYOUT(FXxHash64);

/** A 128-bit hash from XXH128. */
struct FXxHash128
{
	/**
	 * The low 64 bits of the hash in its native representation.
	 *
	 * Use the canonical representation from ToByteArray to serialize or display the hash.
	 */
	uint64 HashLow{};
	/**
	 * The high 64 bits of the hash in its native representation.
	 *
	 * Use the canonical representation from ToByteArray to serialize or display the hash.
	 */
	uint64 HashHigh{};

	[[nodiscard]] CORE_API static FXxHash128 HashBuffer(FMemoryView View);
	[[nodiscard]] CORE_API static FXxHash128 HashBuffer(const void* Data, uint64 Size);
	[[nodiscard]] CORE_API static FXxHash128 HashBuffer(const FCompositeBuffer& Buffer);

	/**
	 * Hash the buffer in parallel in independent chunks, and hash those hashes.
	 *
	 * Use a ChunkSize large enough to cover task overhead, e.g., 256+ KiB.
	 * Hashing the same buffer with different chunk sizes produces a different output hash.
	 */
	[[nodiscard]] CORE_API static FXxHash128 HashBufferChunked(FMemoryView View, uint64 ChunkSize);
	[[nodiscard]] CORE_API static FXxHash128 HashBufferChunked(const void* Data, uint64 Size, uint64 ChunkSize);

	/** Load the hash from its canonical (big-endian) representation. */
	[[nodiscard]] static inline FXxHash128 FromByteArray(const uint8 (&Bytes)[sizeof(uint64[2])])
	{
		uint64 HashBigEndian[2];
		FMemory::Memcpy(&HashBigEndian, Bytes, sizeof(uint64[2]));
		return {NETWORK_ORDER64(HashBigEndian[1]), NETWORK_ORDER64(HashBigEndian[0])};
	}

	/** Store the hash to its canonical (big-endian) representation. */
	inline void ToByteArray(uint8 (&Bytes)[sizeof(uint64[2])]) const
	{
		const uint64 HashBigEndian[2]{NETWORK_ORDER64(HashHigh), NETWORK_ORDER64(HashLow)};
		FMemory::Memcpy(Bytes, &HashBigEndian, sizeof(uint64[2]));
	}

	inline bool operator==(const FXxHash128& B) const
	{
		return HashLow == B.HashLow && HashHigh == B.HashHigh;
	}

	inline bool operator!=(const FXxHash128& B) const
	{
		return HashLow != B.HashLow || HashHigh != B.HashHigh;
	}

	inline bool operator<(const FXxHash128& B) const
	{
		return HashHigh != B.HashHigh ? HashHigh < B.HashHigh : HashLow < B.HashLow;
	}

	inline bool IsZero() const
	{
		return HashHigh == 0 && HashLow == 0;
	}

	friend inline uint32 GetTypeHash(const FXxHash128& Hash)
	{
		return uint32(Hash.HashLow);
	}

	friend inline FArchive& operator<<(FArchive& Ar, FXxHash128& Hash)
	{
		return Ar << Hash.HashLow << Hash.HashHigh;
	}

	template <typename CharType>
	friend inline TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const FXxHash128& Hash)
	{
		uint8 Bytes[16];
		Hash.ToByteArray(Bytes);
		UE::String::BytesToHexLower(MakeArrayView(Bytes), Builder);
		return Builder;
	}
};

/** Calculates a 64-bit hash with XXH3. */
class FXxHash64Builder
{
public:
	inline FXxHash64Builder() { Reset(); }

	FXxHash64Builder(const FXxHash64Builder&) = delete;
	FXxHash64Builder& operator=(const FXxHash64Builder&) = delete;

	CORE_API void Reset();

	CORE_API void Update(FMemoryView View);
	CORE_API void Update(const void* Data, uint64 Size);
	CORE_API void Update(const FCompositeBuffer& Buffer);

	[[nodiscard]] CORE_API FXxHash64 Finalize() const;

private:
	alignas(64) char StateBytes[576];
};

/** Calculates a 128-bit hash with XXH128. */
class FXxHash128Builder
{
public:
	inline FXxHash128Builder() { Reset(); }

	FXxHash128Builder(const FXxHash128Builder&) = delete;
	FXxHash128Builder& operator=(const FXxHash128Builder&) = delete;

	CORE_API void Reset();

	CORE_API void Update(FMemoryView View);
	CORE_API void Update(const void* Data, uint64 Size);
	CORE_API void Update(const FCompositeBuffer& Buffer);

	[[nodiscard]] CORE_API FXxHash128 Finalize() const;

private:
	alignas(64) char StateBytes[576];
};
