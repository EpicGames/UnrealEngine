// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Memory/MemoryView.h"
#include "Templates/RefCounting.h"

template<typename T>
class TIoStatusOr;

/**
 * Reference to buffer data used by I/O dispatcher APIs
 */
class FIoBuffer
{
public:
	enum EAssumeOwnershipTag	{ AssumeOwnership };
	enum ECloneTag				{ Clone };
	enum EWrapTag				{ Wrap };

	CORE_API			FIoBuffer();
	CORE_API explicit	FIoBuffer(uint64 InSize);
	CORE_API			FIoBuffer(const void* Data, uint64 InSize, const FIoBuffer& OuterBuffer);
	CORE_API			FIoBuffer(FMemoryView Memory, const FIoBuffer& OuterBuffer);

	CORE_API			FIoBuffer(EAssumeOwnershipTag,	const void* Data, uint64 InSize);
	CORE_API			FIoBuffer(EAssumeOwnershipTag,	FMemoryView Memory);
	CORE_API			FIoBuffer(ECloneTag,			const void* Data, uint64 InSize);
	CORE_API			FIoBuffer(ECloneTag,			FMemoryView Memory);
	CORE_API			FIoBuffer(EWrapTag,				const void* Data, uint64 InSize);
	CORE_API			FIoBuffer(EWrapTag,				FMemoryView Memory);

	// Note: we currently rely on implicit move constructor, thus we do not declare any
	//		 destructor or copy/assignment operators or copy constructors

	inline const uint8*	Data() const			{ return CorePtr->Data(); }
	inline uint8*		Data()					{ return CorePtr->Data(); }
	inline const uint8*	GetData() const			{ return CorePtr->Data(); }
	inline uint8*		GetData()				{ return CorePtr->Data(); }
	inline uint64		DataSize() const		{ return CorePtr->DataSize(); }
	inline uint64		GetSize() const			{ return CorePtr->DataSize(); }

	inline FMemoryView	GetView() const			{ return MakeMemoryView(CorePtr->Data(), CorePtr->DataSize()); }
	inline FMutableMemoryView GetMutableView()	{ return MakeMemoryView(CorePtr->Data(), CorePtr->DataSize()); }

	inline void			SetSize(uint64 InSize)	{ return CorePtr->SetSize(InSize); }

	inline bool			IsMemoryOwned() const	{ return CorePtr->IsMemoryOwned(); }

	inline void			EnsureOwned() const		{ if (!CorePtr->IsMemoryOwned()) { MakeOwned(); } }

	inline bool			operator!=(const FIoBuffer& Rhs) const { return DataSize() != Rhs.DataSize() || FMemory::Memcmp(GetData(), Rhs.GetData(), DataSize()) != 0; }

	CORE_API void		MakeOwned() const;
	
	/**
	 * Relinquishes control of the internal buffer to the caller and removes it from the FIoBuffer.
	 * This allows the caller to assume ownership of the internal data and prevent it from being deleted along with 
	 * the FIoBuffer.
	 *
	 * NOTE: It is only valid to call this if the FIoBuffer currently owns the internal memory allocation, as the 
	 * point of the call is to take ownership of it. If the FIoBuffer is only wrapping the allocation then it will
	 * return a failed FIoStatus instead.
	 *
	 * @return A status wrapper around the memory pointer. Even if the status is valid the pointer might still be null.
	 */
	[[nodiscard]] CORE_API TIoStatusOr<uint8*> Release();

private:
	/** Core buffer object. For internal use only, used by FIoBuffer

		Contains all state pertaining to a buffer.
	  */
	struct BufCore
	{
					BufCore();
		CORE_API	~BufCore();

		explicit	BufCore(uint64 InSize);
					BufCore(const uint8* InData, uint64 InSize, bool InOwnsMemory);
					BufCore(const uint8* InData, uint64 InSize, const BufCore* InOuter);
					BufCore(ECloneTag, uint8* InData, uint64 InSize);

					BufCore(const BufCore& Rhs) = delete;
		
		BufCore& operator=(const BufCore& Rhs) = delete;

		inline uint8* Data()			{ return DataPtr; }
		inline uint64 DataSize() const	{ return DataSizeLow | (uint64(DataSizeHigh) << 32); }

		//

		CORE_API void SetDataAndSize(const uint8* InData, uint64 InSize);
		CORE_API void SetSize(uint64 InSize);

		void	MakeOwned();

		TIoStatusOr<uint8*> ReleaseMemory();

		inline void SetIsOwned(bool InOwnsMemory)
		{
			if (InOwnsMemory)
			{
				Flags |= OwnsMemory;
			}
			else
			{
				Flags &= ~OwnsMemory;
			}
		}

		inline uint32 AddRef() const
		{
			return uint32(NumRefs.fetch_add(1, std::memory_order_seq_cst) + 1);
		}

		inline uint32 Release() const
		{
			const int32 Refs = NumRefs.fetch_sub(1, std::memory_order_seq_cst);
#if DO_CHECK
			check(Refs > 0);
#endif
			if (Refs == 1)
			{
				delete this;
			}

			return uint32(Refs);
		}

		uint32 GetRefCount() const
		{
			return uint32(NumRefs.load(std::memory_order_relaxed));
		}

		bool IsMemoryOwned() const	{ return Flags & OwnsMemory; }

	private:
		uint8*						DataPtr = nullptr;

		uint32						DataSizeLow = 0;
		mutable std::atomic<int32>	NumRefs = 0;

		// Reference-counted outer "core", used for views into other buffer
		//
		// Ultimately this should probably just be an index into a pool
		TRefCountPtr<const BufCore>	OuterCore;

		// TODO: These two could be packed in the MSB of DataPtr on x64
		uint8		DataSizeHigh = 0;	// High 8 bits of size (40 bits total)
		uint8		Flags = 0;

		enum
		{
			OwnsMemory		= 1 << 0,	// Buffer memory is owned by this instance
			ReadOnlyBuffer	= 1 << 1,	// Buffer memory is immutable
			
			FlagsMask		= (1 << 2) - 1
		};

		void EnsureDataIsResident() {}

		void ClearFlags()
		{
			Flags = 0;
		}
	};

	// Reference-counted "core"
	//
	// Ultimately this should probably just be an index into a pool
	TRefCountPtr<BufCore>	CorePtr;
	
	friend class FIoBufferManager;
};
