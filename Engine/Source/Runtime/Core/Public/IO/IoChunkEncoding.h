// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoOffsetLength.h"
#include "IO/IoStatus.h"
#include "Memory/MemoryView.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"

class FIoBuffer;
struct FIoHash;

using FIoBlockHash = uint32;

/** I/O chunk encryption method. */
enum class EIoEncryptionMethod : uint8
{
	None	= 0,
	AES 	= (1 << 0)
};

/** Defines how the I/O chunk gets encoded into a set of compressed and encrypted block(s). */
struct FIoChunkEncodingParams
{
	FName CompressionFormat = "Oodle";
	FMemoryView EncryptionKey;
	uint32 BlockSize = (64 << 10);
};

/** Parameters for decoding a set of encoded blocks(s). */
struct FIoChunkDecodingParams
	: public FIoChunkEncodingParams
{
	uint64 TotalRawSize = 0;
	uint64 RawOffset = 0;
	uint64 EncodedOffset = 0;
	TConstArrayView<uint32> EncodedBlockSize;
	TConstArrayView<FIoBlockHash> BlockHash;
};

/** Additional options for FIoChunkEncoding::Decode calls */
enum class EIoDecodeFlags : uint8
{
	None		= 0,
	/** Avoid operations that read from the output buffer provided. This option may use more temp scratch memory */
	WriteOnly	= 1 << 0
};
ENUM_CLASS_FLAGS(EIoDecodeFlags);

/** List of supported signature hash functions used when decoding an I/O store chunk block. */
enum class EIoSignatureHashFunction : uint8
{
	Sha1
};

/** Parameters for decoding a single I/O store chunk block. */
struct FIoChunkBlockDecodingParams
{
	FMemoryView					EncryptionKey;
	FMemoryView					BlockHash;
	FName						CompressionFormat;
	EIoSignatureHashFunction	HashFunction = EIoSignatureHashFunction::Sha1;
};

/** Holds the result from decoding a single I/O store chunk block. */
struct FIoChunkBlockDecodeResult
{
	FMutableMemoryView	DecodedBlock;
	EIoErrorCode		ErrorCode = EIoErrorCode::Unknown;
};

/**
 * Encodes data into a set of encrypted and compressed blocks.
 * The chunk encoding information is enocoded into a 16 byte
 * header followed by N block sizes.
 */
class FIoChunkEncoding
{
public:
	static constexpr uint32 ExpectedMagic = 0x2e696f; // .io
	static constexpr uint32 DefaultBlockSize = (64 << 10);
	static constexpr uint32 MaxBlockCount = (1 << 24);
	static constexpr uint64 MaxSize = (uint64(1) << 40);

	/** Header describing the encoded I/O chunk. */
	struct FHeader
	{
		uint64 Magic: 24;
		uint64 RawSize: 40;
		uint64 EncodedSize: 40;
		uint64 BlockSizeExponent: 8;
		uint64 Flags: 8;
		uint64 Pad: 8;

		CORE_API bool IsValid() const;
		CORE_API uint32 GetBlockSize() const;
		CORE_API uint32 GetBlockCount() const;
		CORE_API TConstArrayView<uint32> GetBlocks() const;
		CORE_API uint64 GetTotalHeaderSize() const;

		static CORE_API const FHeader* Decode(FMemoryView HeaderData);
	};

	static_assert(sizeof(FHeader) == 16, "I/O chunk header size mismatch");

	static CORE_API bool Encode(const FIoChunkEncodingParams& Params, FMemoryView RawData, FIoBuffer& OutEncodedData);
	static CORE_API bool Encode(const FIoChunkEncodingParams& Params, FMemoryView RawData, FIoBuffer& OutHeader, FIoBuffer& OutEncodedBlocks);
	static CORE_API bool Decode(const FIoChunkDecodingParams& Params, FMemoryView EncodedBlocks, FMutableMemoryView OutRawData, EIoDecodeFlags Flags = EIoDecodeFlags::None);
	static CORE_API bool Decode(FMemoryView EncodedData, FName CompressionFormat, FMemoryView EncryptionKey, FMutableMemoryView OutRawData, uint64 Offset = 0);
	/**
	 * Decode a single I/O store chunk block.
	 * 
	 * @param Params			Controls whether this function:
	 *								- computes and verifys the block signature using the provided block hash
	 *								- decrypts the block using the provided encryption key
	 *								- decompress the source buffer to the destinatino buffer
	 * @param EncodedBlock		Memory view of the encoded source buffer. Assumes the underlying size is aligned to AES block size.
	 * @param OutDecodedBlock	Memory view of the destination buffer.
	 *
	 * @note This function does not copy the bytes from the encoded block unless it is compressed.
	 */
	static CORE_API FIoChunkBlockDecodeResult DecodeBlock(const FIoChunkBlockDecodingParams Params, FMutableMemoryView EncodedBlock, FMutableMemoryView OutDecodedBlock);

	static CORE_API TIoStatusOr<FIoOffsetAndLength> GetChunkRange(uint64 TotalRawSize, uint32 RawBlockSize, TConstArrayView<uint32> EncodedBlockSize, uint64 RawOffset, uint64 RawSize);
	static CORE_API TIoStatusOr<FIoOffsetAndLength> GetChunkRange(const FIoChunkDecodingParams& Params, uint64 RawSize);
	static CORE_API uint64 GetTotalEncodedSize(TConstArrayView<uint32> EncodedBlockSize);
	static CORE_API FIoBlockHash HashBlock(FMemoryView Block);
};

/**
 * Callback used when decoding I/O store chunk blocks asynchronously. The optional out
 * paramater can be used to schedule more decode requests without recursively calling IIoChunkBlockDecoder::Enqueue.
 */
using FIoBlockDecoded = TUniqueFunction<void(FIoChunkBlockDecodeResult&&, struct FIoChunkBlockDecodeRequest&)>;

struct FIoChunkBlockDecodeRequest
{
	bool						IsValid() const { return OnDecoded.IsSet(); }

	FIoChunkBlockDecodingParams Params;
	FMutableMemoryView			EncodedBlock;
	FMutableMemoryView			DecodedBlock;
	FIoBlockDecoded 			OnDecoded;
};

/** Interface for decoding I/O store chunk blocks asynchronously. */
class IIoChunkBlockDecoder
{
public:
	virtual			~IIoChunkBlockDecoder() = default;
	virtual void	Enqueue(FIoChunkBlockDecodeRequest&& DecodeRequest) = 0;
};

/** Provides access to the chunk block decoder. */
class FIoChunkBlockDecoder
{
public:
	CORE_API static void					Set(IIoChunkBlockDecoder& Instance);
	CORE_API static IIoChunkBlockDecoder&	Get();
	CORE_API static void					Release();

private:
	static IIoChunkBlockDecoder* Instance;
};
