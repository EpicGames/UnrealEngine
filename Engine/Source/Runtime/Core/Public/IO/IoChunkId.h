// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringFwd.h"
#include "Memory/MemoryView.h"
#include "Misc/ByteSwap.h"
#include "String/BytesToHex.h"

class FArchive;
class FCbFieldView;
class FCbWriter;
class FPackageId;
class FIoContainerId;

/**
 * Addressable chunk types.
 * 
 * The enumerators have explicitly defined values here to encourage backward/forward
 * compatible updates. 
 * 
 * Also note that for certain discriminators, Zen Store will assume certain things
 * about the structure of the chunk ID so changes must be made carefully.
 * 
 */
enum class EIoChunkType : uint8
{
	Invalid = 0,
	ExportBundleData = 1,
	BulkData = 2,
	OptionalBulkData = 3,
	MemoryMappedBulkData = 4,
	ScriptObjects = 5,
	ContainerHeader = 6,
	ExternalFile = 7,
	ShaderCodeLibrary = 8,
	ShaderCode = 9,
	PackageStoreEntry = 10,
	DerivedData = 11,
	EditorDerivedData = 12,
	PackageResource = 13,
	MAX
};

CORE_API FString LexToString(const EIoChunkType Type);

inline bool IsBulkDataType(const EIoChunkType Type)
{
	if (Type == EIoChunkType::BulkData || Type == EIoChunkType::OptionalBulkData || Type == EIoChunkType::MemoryMappedBulkData)
	{
		return true;
	}
	else
	{
		return false;
	}
}

/**
 * Identifier to a chunk of data.
 */
class FIoChunkId
{
public:
	CORE_API static const FIoChunkId InvalidChunkId;

	friend uint32 GetTypeHash(FIoChunkId InId)
	{
		uint32 Hash = 5381;
		for (int i = 0; i < sizeof Id; ++i)
		{
			Hash = Hash * 33 + InId.Id[i];
		}
		return Hash;
	}

	friend CORE_API FArchive& operator<<(FArchive& Ar, FIoChunkId& ChunkId);
	friend CORE_API FCbWriter& operator<<(FCbWriter& Writer, const FIoChunkId& ChunkId);
	friend CORE_API bool LoadFromCompactBinary(FCbFieldView Field, FIoChunkId& OutChunkId);

	template <typename CharType>
	friend TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const FIoChunkId& ChunkId)
	{
		UE::String::BytesToHexLower(ChunkId.Id, Builder);
		return Builder;
	}

	CORE_API void ToString(FString& Output) const;
	friend CORE_API FString LexToString(const FIoChunkId& Id);

	inline bool operator ==(const FIoChunkId& Rhs) const
	{
		return 0 == FMemory::Memcmp(Id, Rhs.Id, sizeof Id);
	}

	inline bool operator !=(const FIoChunkId& Rhs) const
	{
		return !(*this == Rhs);
	}

	inline bool operator <(const FIoChunkId& Rhs) const
	{
		return FMemory::Memcmp(Id, Rhs.Id, sizeof Id) < 0;
	}

	inline bool operator >(const FIoChunkId& Rhs) const
	{
		return FMemory::Memcmp(Id, Rhs.Id, sizeof Id) > 0;
	}

	void Set(const void* InIdPtr, SIZE_T InSize)
	{
		check(InSize == sizeof Id);
		FMemory::Memcpy(Id, InIdPtr, sizeof Id);
	}

	void Set(FMemoryView InView)
	{
		check(InView.GetSize() == sizeof Id);
		FMemory::Memcpy(Id, InView.GetData(), sizeof Id);
	}

	inline bool IsValid() const
	{
		return *this != InvalidChunkId;
	}

	inline const uint8* GetData() const { return Id; }
	inline uint32		GetSize() const { return sizeof Id; }

	EIoChunkType GetChunkType() const
	{
		return static_cast<EIoChunkType>(Id[11]);
	}

	friend class FIoStoreReaderImpl;

	/**
	 * Creates an I/O chunk ID from a 24 character long hexadecimal string.
	 * @return The corresponding I/O chunk ID or an invalid ID if the input string is not in the correct format.
	 */
	CORE_API static FIoChunkId FromHex(FStringView Hex);

private:
	static inline FIoChunkId CreateEmptyId()
	{
		FIoChunkId ChunkId;
		uint8 Data[12] = { 0 };
		ChunkId.Set(Data, sizeof Data);

		return ChunkId;
	}

	uint8	Id[12];
};


/** Creates a chunk identifier (generic -- prefer specialized versions where possible). */
inline FIoChunkId CreateIoChunkId(uint64 ChunkId, uint16 ChunkIndex, EIoChunkType IoChunkType)
{
	checkSlow(IoChunkType != EIoChunkType::ExternalFile);	// Use CreateExternalFileChunkId() instead
	checkfSlow(IsBulkDataType(IoChunkType) == false, TEXT("Bulkdata types should call CreateBulkDataIoChunkId instead"));

	uint8 Data[12] = {0};

	*reinterpret_cast<uint64*>(&Data[0]) = ChunkId;
	*reinterpret_cast<uint16*>(&Data[8]) = NETWORK_ORDER16(ChunkIndex);
	*reinterpret_cast<uint8*>(&Data[11]) = static_cast<uint8>(IoChunkType);

	FIoChunkId IoChunkId;
	IoChunkId.Set(Data, 12);

	return IoChunkId;
}



/**
 * Used to create a FIoChunkId that references a bulkdata chunk.
 * 
 * TODO Cannot use FBulkDataChunkGroup directly here as it is in CoreUObject
 * 
 * @param ChunkId		FPackageId
 * @param ChunkIndex	MultiOutputIindex [0 = disabled, 1 enabled]
 * @param ChunkGroup	FBulkDataCookedIndex
 * @param IoChunkType	EIoChunkType
 */
inline FIoChunkId CreateBulkDataIoChunkId(uint64 ChunkId, uint16 ChunkIndex, uint8 ChunkGroup, EIoChunkType IoChunkType)
{
	checkSlow(IoChunkType != EIoChunkType::ExternalFile);	// Use CreateExternalFileChunkId() instead
	checkfSlow(IsBulkDataType(IoChunkType), TEXT("CreateBulkDataIoChunkId is only intended for bulkdata types"));
	
	uint8 Data[12] = { 0 };

	*reinterpret_cast<uint64*>(&Data[0]) = ChunkId;
	*reinterpret_cast<uint16*>(&Data[8]) = NETWORK_ORDER16(ChunkIndex);
	*reinterpret_cast<uint8*>(&Data[10]) = ChunkGroup;
	*reinterpret_cast<uint8*>(&Data[11]) = static_cast<uint8>(IoChunkType);

	FIoChunkId IoChunkId;
	IoChunkId.Set(Data, 12);

	return IoChunkId;
}

/** Returns a package data I/O chunk ID for the specified package ID. */
CORE_API FIoChunkId CreatePackageDataChunkId(const FPackageId& PackageId);

/** Returns a file data I/O chunk ID for the specified filename. */
CORE_API FIoChunkId CreateExternalFileChunkId(const FStringView Filename);

/** Returns a container header I/O chunk ID. */
CORE_API FIoChunkId CreateContainerHeaderChunkId(const FIoContainerId& ContainerId);
