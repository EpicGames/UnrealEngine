// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "IO/IoContainerId.h"
#include "IO/PackageId.h"
#include "Serialization/MappedName.h"
#include "UObject/NameBatchSerialization.h"

class FArchive;
struct FShaderHash;

/**
 * Package store entry array view.
 */
template<typename T>
class TFilePackageStoreEntryCArrayView
{
	const uint32 ArrayNum = 0;
	const uint32 OffsetToDataFromThis = 0;

public:
	inline uint32 Num() const { return ArrayNum; }

	inline const T* Data() const { return (T*)((char*)this + OffsetToDataFromThis); }
	inline T* Data() { return (T*)((char*)this + OffsetToDataFromThis); }

	inline const T* begin() const { return Data(); }
	inline T* begin() { return Data(); }

	inline const T* end() const { return Data() + ArrayNum; }
	inline T* end() { return Data() + ArrayNum; }

	inline const T& operator[](uint32 Index) const { return Data()[Index]; }
	inline T& operator[](uint32 Index) { return Data()[Index]; }
};

/**
 * File based package store entry
 */
struct FFilePackageStoreEntry
{
	TFilePackageStoreEntryCArrayView<FPackageId> ImportedPackages;
	TFilePackageStoreEntryCArrayView<FShaderHash> ShaderMapHashes;
};

struct FIoContainerHeaderPackageRedirect
{
	FPackageId SourcePackageId;
	FPackageId TargetPackageId;
	FMappedName SourcePackageName;

	CORE_API friend FArchive& operator<<(FArchive& Ar, FIoContainerHeaderPackageRedirect& PackageRedirect);
};

struct FIoContainerHeaderLocalizedPackage
{
	FPackageId SourcePackageId;
	FMappedName SourcePackageName;

	CORE_API friend FArchive& operator<<(FArchive& Ar, FIoContainerHeaderLocalizedPackage& LocalizedPackage);
};

struct FFilePackageStoreEntrySoftReferences
{
	TFilePackageStoreEntryCArrayView<uint32> Indices;
};

struct FIoContainerHeaderSoftPackageReferences
{
	CORE_API void Empty();

	// Deduplicated list of soft referenced package IDs for all packages in the container.
	TArray<FPackageId> PackageIds;
	// Indices into the package ID list for all packages in the container serialized as array views.
	TArray<uint8> PackageIndices; 
	// Flag indicating whether any soft package references exists.
	bool bContainsSoftPackageReferences = false;

	// Transient flag indicating that the soft package references were loaded at runtime
	bool bLoadedSoftPackageReferences = false;

	CORE_API friend FArchive& operator<<(FArchive& Ar, FIoContainerHeaderSoftPackageReferences& SoftPackageReferences);
};

struct FIoContainerHeaderSerialInfo
{
	int64 Offset = -1;
	int64 Size = -1;

	CORE_API friend FArchive& operator<<(FArchive& Ar, FIoContainerHeaderSerialInfo& SerialInfo);
};

enum class EIoContainerHeaderVersion : uint32
{
	Initial = 0,
	LocalizedPackages = 1,
	OptionalSegmentPackages = 2,
	NoExportInfo = 3,
	SoftPackageReferences = 4,
	SoftPackageReferencesOffset = 5,

	LatestPlusOne,
	Latest = LatestPlusOne - 1
};

struct FIoContainerHeader
{
	enum
	{
		Signature = 0x496f436e
	};

	FIoContainerId ContainerId;
	TArray<FPackageId> PackageIds;
	TArray<uint8> StoreEntries; //FPackageStoreEntry[PackageIds.Num()]
	TArray<FPackageId> OptionalSegmentPackageIds;
	TArray<uint8> OptionalSegmentStoreEntries; //FPackageStoreEntry[OptionalSegmentPackageIds.Num()]
	TArray<FDisplayNameEntryId> RedirectsNameMap;
	TArray<FIoContainerHeaderLocalizedPackage> LocalizedPackages;
	TArray<FIoContainerHeaderPackageRedirect> PackageRedirects;
	FIoContainerHeaderSerialInfo SoftPackageReferencesSerialInfo;
	FIoContainerHeaderSoftPackageReferences SoftPackageReferences;

	CORE_API friend FArchive& operator<<(FArchive& Ar, FIoContainerHeader& ContainerHeader);
};
