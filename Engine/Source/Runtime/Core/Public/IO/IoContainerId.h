// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/NameTypes.h"

class FArchive;
class FStructuredArchiveSlot;
class FCbFieldView;
class FCbWriter;

/**
 * Container ID.
 */
class FIoContainerId
{
public:
	inline FIoContainerId() = default;
	inline FIoContainerId(const FIoContainerId& Other) = default;
	inline FIoContainerId(FIoContainerId&& Other) = default;
	inline FIoContainerId& operator=(const FIoContainerId& Other) = default;

	CORE_API static FIoContainerId FromName(const FName& Name);

	uint64 Value() const
	{
		return Id;
	}

	inline bool IsValid() const
	{ 
		return Id != InvalidId;
	}

	inline bool operator<(FIoContainerId Other) const
	{
		return Id < Other.Id;
	}

	inline bool operator==(FIoContainerId Other) const
	{
		return Id == Other.Id;
	}

	inline bool operator!=(FIoContainerId Other) const
	{
		return Id != Other.Id;
	}

	inline friend uint32 GetTypeHash(const FIoContainerId& In)
	{
		return uint32(In.Id);
	}

	CORE_API friend FArchive& operator<<(FArchive& Ar, FIoContainerId& ContainerId);

	CORE_API friend void operator<<(FStructuredArchiveSlot Slot, FIoContainerId& ContainerId);

	CORE_API friend FCbWriter& operator<<(FCbWriter& Writer, const FIoContainerId& ContainerId);

	CORE_API friend FString LexToString(const FIoContainerId& ContainerId);

	CORE_API friend bool LoadFromCompactBinary(FCbFieldView Field, FIoContainerId& OutContainerId);

private:
	inline explicit FIoContainerId(const uint64 InId)
		: Id(InId) { }

	static constexpr uint64 InvalidId = uint64(-1);

	uint64 Id = InvalidId;
};
