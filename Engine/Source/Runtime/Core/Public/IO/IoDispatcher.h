// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PlatformFile.h"
#include "HAL/UnrealMemory.h"
#include "IO/IoBuffer.h"
#include "IO/IoChunkId.h"
#include "IO/IoContainerId.h"
#include "IO/IoDispatcherPriority.h"
#include "IO/IoHash.h"
#include "IO/IoStatus.h"
#include "Logging/LogMacros.h"
#include "Math/NumericLimits.h"
#include "Memory/MemoryFwd.h"
#include "Memory/MemoryView.h"
#include "Misc/AES.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Build.h"
#include "Misc/ByteSwap.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "Misc/IEngineCrypto.h"
#include "Misc/SecureHash.h"
#include "Serialization/Archive.h"
#include "Serialization/FileRegions.h"
#include "String/BytesToHex.h"
#include "Tasks/Task.h"
#include "Templates/Function.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FEvent;
class FIoBatchImpl;
class FIoDirectoryIndexReaderImpl;
class FIoDispatcher;
class FIoDispatcherImpl;
class FIoRequest;
class FIoRequestImpl;
class FIoStoreEnvironment;
class FIoStoreReader;
class FIoStoreReaderImpl;
class FPackageId;
class IMappedFileHandle;
class IMappedFileRegion;
struct FFileRegion;
struct IIoDispatcherBackend;
struct FIoOffsetAndLength;
template <typename CharType> class TStringBuilderBase;
template <typename OptionalType> struct TOptional;

CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogIoDispatcher, Log, All);

/** Helper used to manage creation of I/O store file handles etc. */
class
UE_DEPRECATED(5.8, "FIoStoreEnvironment is not used and is deprecated.")
FIoStoreEnvironment
{
public:
	CORE_API FIoStoreEnvironment();
	CORE_API ~FIoStoreEnvironment();

	CORE_API void InitializeFileEnvironment(FStringView InPath, int32 InOrder = 0);

	const FString& GetPath() const { return Path; }
	int32 GetOrder() const { return Order; }

private:
	FString			Path;
	int32			Order = 0;
};

/** Additional I/O dispatcher read option flags. */
enum class EIoReadOptionsFlags : uint32
{
	None = 0,
	/**
	 * Use this flag to inform the decompressor that the memory is uncached or write-combined and therefore the usage of staging might be needed if reading directly from the original memory
	 */
	HardwareTargetBuffer = 1 << 0,
};
ENUM_CLASS_FLAGS(EIoReadOptionsFlags);

/** I/O dispatcher read options. */
class FIoReadOptions
{
public:
	FIoReadOptions() = default;

	FIoReadOptions(uint64 InOffset, uint64 InSize)
		: RequestedOffset(InOffset)
		, RequestedSize(InSize)
	{ }
	
	FIoReadOptions(uint64 InOffset, uint64 InSize, void* InTargetVa)
		: RequestedOffset(InOffset)
		, RequestedSize(InSize)
		, TargetVa(InTargetVa)
	{ }

	FIoReadOptions(uint64 InOffset, uint64 InSize, void* InTargetVa, EIoReadOptionsFlags InFlags)
		: RequestedOffset(InOffset)
		, RequestedSize(InSize)
		, TargetVa(InTargetVa)
		, Flags(InFlags)
	{ }

	~FIoReadOptions() = default;

	/** Set the requested range within the I/O store chunk. */
	void SetRange(uint64 Offset, uint64 Size)
	{
		RequestedOffset = Offset;
		RequestedSize	= Size;
	}

	/** Set the target address where to write the decoded I/O store chunk data. */
	void SetTargetVa(void* InTargetVa)
	{
		TargetVa = InTargetVa;
	}

	/** Set additional read option flags. */
	void SetFlags(EIoReadOptionsFlags InValue)
	{
		Flags = InValue;
	}

	/** Returns the offset within the I/O store chunk. */
	uint64 GetOffset() const
	{
		return RequestedOffset;
	}

	/** Returns the range size to read. */
	uint64 GetSize() const
	{
		return RequestedSize;
	}

	/** Returns the target address. */
	void* GetTargetVa() const
	{
		return TargetVa;
	}

	/** Returns the read option flags. */
	EIoReadOptionsFlags GetFlags() const
	{
		return Flags;
	}

private:
	uint64	RequestedOffset = 0;
	uint64	RequestedSize = ~uint64(0);
	void* TargetVa = nullptr;
	EIoReadOptionsFlags Flags = EIoReadOptionsFlags::None;
};

/** Handle to an I/O dispatcher read request. */
class FIoRequest final
{
public:
	FIoRequest() = default;
	CORE_API ~FIoRequest();

	CORE_API FIoRequest(const FIoRequest& Other);
	CORE_API FIoRequest(FIoRequest&& Other);
	CORE_API FIoRequest& operator=(const FIoRequest& Other);
	CORE_API FIoRequest& operator=(FIoRequest&& Other);
	CORE_API FIoStatus						Status() const;
	CORE_API const FIoBuffer*				GetResult() const;
	CORE_API const FIoBuffer&				GetResultOrDie() const;
	CORE_API void							Cancel();
	CORE_API void							UpdatePriority(int32 NewPriority);
	CORE_API void							Release();

private:
	FIoRequestImpl* Impl = nullptr;

	explicit FIoRequest(FIoRequestImpl* InImpl);

	friend class FIoDispatcher;
	friend class FIoDispatcherImpl;
	friend class FIoDispatcherInternal;
	friend class FIoBatch;
};

using FIoReadCallback = TFunction<void(TIoStatusOr<FIoBuffer>)>;

/** Converts filesystem priority flags to I/O dispatcher priority. */
inline int32 ConvertToIoDispatcherPriority(EAsyncIOPriorityAndFlags AIOP)
{
	int32 AIOPriorityToIoDispatcherPriorityMap[] = {
		IoDispatcherPriority_Min,
		IoDispatcherPriority_Low,
		IoDispatcherPriority_Medium - 1,
		IoDispatcherPriority_Medium,
		IoDispatcherPriority_High,
		IoDispatcherPriority_Max
	};
	static_assert(AIOP_NUM == UE_ARRAY_COUNT(AIOPriorityToIoDispatcherPriorityMap), "IoDispatcher and AIO priorities mismatch");
	return AIOPriorityToIoDispatcherPriorityMap[AIOP & AIOP_PRIORITY_MASK];
}

/** Enables dispatching multiple I/O dispatcher request(s) as a single request. */
class FIoBatch final
{
	friend class FIoDispatcher;
	friend class FIoDispatcherImpl;
	friend class FIoRequestStats;

public:
	CORE_API FIoBatch();
	CORE_API FIoBatch(FIoBatch&& Other);
	CORE_API ~FIoBatch();
	CORE_API FIoBatch& operator=(FIoBatch&& Other);
	CORE_API FIoRequest Read(const FIoChunkId& Chunk, FIoReadOptions Options, int32 Priority);
	CORE_API FIoRequest ReadWithCallback(const FIoChunkId& ChunkId, const FIoReadOptions& Options, int32 Priority, FIoReadCallback&& Callback);

	CORE_API void Issue();
	CORE_API void IssueWithCallback(TFunction<void()>&& Callback);
	CORE_API void IssueAndTriggerEvent(FEvent* Event);
	CORE_API void IssueAndDispatchSubsequents(FGraphEventRef Event);

private:
	FIoBatch(FIoDispatcherImpl& InDispatcher);
	FIoRequestImpl* ReadInternal(const FIoChunkId& ChunkId, const FIoReadOptions& Options, int32 Priority);

	FIoDispatcherImpl*	Dispatcher;
	FIoRequestImpl*		HeadRequest = nullptr;
	FIoRequestImpl*		TailRequest = nullptr;
};

/** File and region handles for a memory mapped I/O store chunk. */
struct FIoMappedRegion
{
	IMappedFileHandle* MappedFileHandle = nullptr;
	IMappedFileRegion* MappedFileRegion = nullptr;
};

/** Holds information about a mounted container. */
struct
UE_DEPRECATED(5.8, "FIoDispatcherMountedContainer is deprecated.")
FIoDispatcherMountedContainer
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FIoStoreEnvironment Environment;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	FIoContainerId ContainerId;
};

/** Holds information to where a signature error occurred. */
struct FIoSignatureError
{
	FString ContainerName;
	int32 BlockIndex = INDEX_NONE;
	FSHAHash ExpectedHash;
	FSHAHash ActualHash;
};

/** Signature error delegate. */
DECLARE_MULTICAST_DELEGATE_OneParam(FIoSignatureErrorDelegate, const FIoSignatureError&);

struct
UE_DEPRECATED(5.8, "FIoSignatureErrorEvent is deprecated. Use FIoSignatureError.")
FIoSignatureErrorEvent
{
	FCriticalSection CriticalSection;
	FIoSignatureErrorDelegate SignatureErrorDelegate;
};

/**
 * The I/O dispatcher enables reading data addressed with chunk identifiers (@see FIoChunkId).
 * Multiple backends can be mounted enabling reading data from multiple storage mediums such
 * as the fileystem or via HTTP.
 */
class FIoDispatcher final
{
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	using FIoContainerMountedEvent UE_DEPRECATED(5.8, "FIoContainerMountedEvent is deprecated") =
		TMulticastDelegate<void(const FIoDispatcherMountedContainer&)>;
	using FIoContainerUnmountedEvent UE_DEPRECATED(5.8, "FIoContainerUnmountedEvent is deprecated") =
		TMulticastDelegate<void(const FIoDispatcherMountedContainer&)>;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	CORE_API						FIoDispatcher();
	CORE_API						~FIoDispatcher();
									FIoDispatcher(const FIoDispatcher&) = delete;
									FIoDispatcher& operator=(const FIoDispatcher&) = delete;

	CORE_API void					Mount(TSharedRef<IIoDispatcherBackend> Backend, int32 Priority = 0);

	CORE_API FIoBatch				NewBatch();

	CORE_API TIoStatusOr<FIoMappedRegion> OpenMapped(const FIoChunkId& ChunkId, const FIoReadOptions& Options);

	// Polling methods
	CORE_API bool					DoesChunkExist(const FIoChunkId& ChunkId) const;
	CORE_API TIoStatusOr<uint64>	GetSizeForChunk(const FIoChunkId& ChunkId) const;
	CORE_API int64					GetTotalLoaded() const;

	// Events
	CORE_API FIoSignatureErrorDelegate& OnSignatureError();

	static CORE_API bool			IsInitialized();
	static CORE_API FIoStatus		Initialize();
	static CORE_API void			InitializePostSettings();
	static CORE_API void			Shutdown();
	static CORE_API FIoDispatcher&	Get();

private:
	CORE_API FUtf8StringView		GetFilename(const FIoChunkId& ChunkId, FUtf8StringBuilderBase& OutFilename, FUtf8StringView& OutContainerName) const;
	CORE_API bool					DoesChunkExist(const FIoChunkId& ChunkId, const FIoOffsetAndLength& ChunkRange) const;
	CORE_API TIoStatusOr<uint64>	GetSizeForChunk(const FIoChunkId& ChunkId, const FIoOffsetAndLength& ChunkRange, uint64& OutAvailable) const;
	CORE_API void					MountContainerMeta(const FString& FilePath);
	

	FIoDispatcherImpl* Impl = nullptr;

	friend class FIoRequest;
	friend class FIoBatch;
	friend class FIoQueue;
	friend class FBulkData;
	friend class FIoDispatcherInternal;
};

/** Represents a handle into the I/O store directory index. */
class FIoDirectoryIndexHandle
{
	static constexpr uint32 InvalidHandle = ~uint32(0);
	static constexpr uint32 RootHandle = 0;

public:
	FIoDirectoryIndexHandle() = default;

	inline bool IsValid() const
	{
		return Handle != InvalidHandle;
	}

	inline bool operator<(FIoDirectoryIndexHandle Other) const
	{
		return Handle < Other.Handle;
	}

	inline bool operator==(FIoDirectoryIndexHandle Other) const
	{
		return Handle == Other.Handle;
	}

	inline friend uint32 GetTypeHash(FIoDirectoryIndexHandle InHandle)
	{
		return InHandle.Handle;
	}

	inline uint32 ToIndex() const
	{
		return Handle;
	}

	static inline FIoDirectoryIndexHandle FromIndex(uint32 Index)
	{
		return FIoDirectoryIndexHandle(Index);
	}

	static inline FIoDirectoryIndexHandle RootDirectory()
	{
		return FIoDirectoryIndexHandle(RootHandle);
	}

	static inline FIoDirectoryIndexHandle Invalid()
	{
		return FIoDirectoryIndexHandle(InvalidHandle);
	}

private:
	FIoDirectoryIndexHandle(uint32 InHandle)
		: Handle(InHandle) { }

	uint32 Handle = InvalidHandle;
};

using FDirectoryIndexVisitorFunction = TFunctionRef<bool(FStringView, const uint32)>;

/**
 * The directory index reader enables reading filenames mapped from chunk identifiers.
 * Note that the directory index is still considered metadata and is not available in
 * the game runtime.
 */
class FIoDirectoryIndexReader
{
public:
	CORE_API FIoDirectoryIndexReader();
	CORE_API ~FIoDirectoryIndexReader();
	CORE_API FIoStatus Initialize(TConstArrayView<uint8> InBuffer, FAES::FAESKey InDecryptionKey);

	CORE_API const FString& GetMountPoint() const;
	CORE_API FIoDirectoryIndexHandle GetChildDirectory(FIoDirectoryIndexHandle Directory) const;
	CORE_API FIoDirectoryIndexHandle GetNextDirectory(FIoDirectoryIndexHandle Directory) const;
	CORE_API FIoDirectoryIndexHandle GetFile(FIoDirectoryIndexHandle Directory) const;
	CORE_API FIoDirectoryIndexHandle GetNextFile(FIoDirectoryIndexHandle File) const;
	CORE_API FStringView GetDirectoryName(FIoDirectoryIndexHandle Directory) const;
	CORE_API FStringView GetFileName(FIoDirectoryIndexHandle File) const;
	CORE_API uint32 GetFileData(FIoDirectoryIndexHandle File) const;

	CORE_API bool IterateDirectoryIndex(FIoDirectoryIndexHandle Directory, FStringView Path, FDirectoryIndexVisitorFunction Visit) const;

private:
	UE_NONCOPYABLE(FIoDirectoryIndexReader);

	FIoDirectoryIndexReaderImpl* Impl;
};

/** I/O store container flags. */
enum class EIoContainerFlags : uint8
{
	None,
	Compressed	= (1 << 0),
	Encrypted	= (1 << 1),
	Signed		= (1 << 2),
	Indexed		= (1 << 3),
	OnDemand	= (1 << 4),
	Last		= OnDemand
};
ENUM_CLASS_FLAGS(EIoContainerFlags);

CORE_API FStringBuilderBase& operator<<(FStringBuilderBase& Sb, EIoContainerFlags Flags);
CORE_API FString LexToString(EIoContainerFlags Flags);

/** Settings used when creating I/O store container files (.ucas). */
struct FIoContainerSettings
{
	FIoContainerId ContainerId;
	EIoContainerFlags ContainerFlags = EIoContainerFlags::None;
	FGuid EncryptionKeyGuid;
	FAES::FAESKey EncryptionKey;
	FRSAKeyHandle SigningKey;
	bool bGenerateDiffPatch = false;

	bool IsCompressed() const
	{
		return !!(ContainerFlags & EIoContainerFlags::Compressed);
	}

	bool IsEncrypted() const
	{
		return !!(ContainerFlags & EIoContainerFlags::Encrypted);
	}

	bool IsSigned() const
	{
		return !!(ContainerFlags & EIoContainerFlags::Signed);
	}

	bool IsIndexed() const
	{
		return !!(ContainerFlags & EIoContainerFlags::Indexed);
	}
	
	bool IsOnDemand() const
	{
		return !!(ContainerFlags & EIoContainerFlags::OnDemand);
	}
};

/** Holds information about an I/O store chunk found in the container TOC. */
struct FIoStoreTocChunkInfo
{
	FIoChunkId Id;
	FIoHash ChunkHash;
	FString FileName;
	uint64 Offset;
	uint64 OffsetOnDisk;
	uint64 Size;
	uint64 SizeOnDisk;
	uint64 CompressedSize;
	uint32 NumCompressedBlocks;
	int32 PartitionIndex;
	EIoChunkType ChunkType;
	bool bHasValidFileName;
	bool bForceUncompressed;
	bool bIsMemoryMapped;
	bool bIsCompressed;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS // Compilers can complain about deprecated members in compiler generated code
	FIoStoreTocChunkInfo() = default;
	FIoStoreTocChunkInfo(const FIoStoreTocChunkInfo&) = default;
	FIoStoreTocChunkInfo(FIoStoreTocChunkInfo&&) = default;
	FIoStoreTocChunkInfo& operator=(FIoStoreTocChunkInfo&) = default;
	FIoStoreTocChunkInfo& operator=(FIoStoreTocChunkInfo&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

/** Holds information about an I/O store chunk. */
struct FIoStoreTocCompressedBlockInfo
{
	uint64 Offset;
	uint32 CompressedSize;
	uint32 UncompressedSize;
	uint8 CompressionMethodIndex;
};

/** Holds information about an I/O store compression block. */
struct FIoStoreCompressedBlockInfo
{
	/** Name of the method used to compress the block. */
	FName CompressionMethod;
	/** The size of relevant data in the block (i.e. what you pass to decompress). */
	uint32 CompressedSize;
	/** The size of the _block_ after decompression. This is not adjusted for any FIoReadOptions used. */
	uint32 UncompressedSize;
	/** The size of the data this block takes in IoBuffer (i.e. after padding for decryption). */
	uint32 AlignedSize;
	/** Where in IoBuffer this block starts. */
	uint64 OffsetInBuffer;
};

/** Holds information about a decoded I/O store chunk. */
struct FIoStoreCompressedReadResult
{
	/** The buffer containing the chunk. */
	FIoBuffer IoBuffer;

	/** Info about the blocks that the chunk is split up into. */
	TArray<FIoStoreCompressedBlockInfo> Blocks;
	// There is where the data starts in IoBuffer (for when you pass in a data range via FIoReadOptions)
	uint64 UncompressedOffset = 0;
	// This is the total size requested via FIoReadOptions. Notably, if you requested a narrow range, you could
	// add up all the block uncompressed sizes and it would be larger than this.
	uint64 UncompressedSize = 0;
	// This is the total size of compressed data, which is less than IoBuffer size due to padding for decryption.
	uint64 TotalCompressedSize = 0;
};

/** The I/O store reader provides non-runtime convenience functionality for reading I/O store container files. */
class FIoStoreReader
{
public:
	CORE_API FIoStoreReader();
	CORE_API ~FIoStoreReader();

	[[nodiscard]] CORE_API FIoStatus Initialize(FStringView ContainerPath, const TMap<FGuid, FAES::FAESKey>& InDecryptionKeys);
	CORE_API FIoContainerId GetContainerId() const;
	CORE_API uint32 GetVersion() const;
	CORE_API EIoContainerFlags GetContainerFlags() const;
	CORE_API FGuid GetEncryptionKeyGuid() const;
	CORE_API int32 GetChunkCount() const;
	CORE_API FString GetContainerName() const; // The container name is the base filename of ContainerPath, e.g. "global".

	CORE_API void EnumerateChunks(TFunction<bool(FIoStoreTocChunkInfo&&)>&& Callback) const;
	CORE_API TIoStatusOr<FIoStoreTocChunkInfo> GetChunkInfo(const FIoChunkId& Chunk) const;
	CORE_API TIoStatusOr<FIoStoreTocChunkInfo> GetChunkInfo(const uint32 TocEntryIndex) const;

	// Reads the chunk off the disk, decrypting/decompressing as necessary.
	CORE_API TIoStatusOr<FIoBuffer> Read(const FIoChunkId& Chunk, const FIoReadOptions& Options) const;
	
	// As Read(), except returns a task that will contain the result after a .Wait/.BusyWait.
	CORE_API UE::Tasks::TTask<TIoStatusOr<FIoBuffer>> ReadAsync(const FIoChunkId& Chunk, const FIoReadOptions& Options) const;

	// Reads and decrypts if necessary the compressed blocks, but does _not_ decompress them. The totality of the data is stored
	// in FIoStoreCompressedReadResult::FIoBuffer as a contiguous buffer, however each block is padded during encryption, so
	// either use FIoStoreCompressedBlockInfo::AlignedSize to advance through the buffer, or use FIoStoreCompressedBlockInfo::OffsetInBuffer
	// directly.
	CORE_API TIoStatusOr<FIoStoreCompressedReadResult> ReadCompressed(const FIoChunkId& Chunk, const FIoReadOptions& Options, bool bDecrypt = true) const;

	CORE_API const FIoDirectoryIndexReader& GetDirectoryIndexReader() const;

	CORE_API void GetFilenamesByBlockIndex(const TArray<int32>& InBlockIndexList, TArray<FString>& OutFileList) const;
	CORE_API void GetFilenames(TArray<FString>& OutFileList) const;

	CORE_API uint64 GetPartitionSize() const;
	CORE_API uint32 GetCompressionBlockSize() const;
	CORE_API const TArray<FName>& GetCompressionMethods() const;
	CORE_API void EnumerateCompressedBlocks(TFunction<bool(const FIoStoreTocCompressedBlockInfo&)>&& Callback) const;
	CORE_API void EnumerateCompressedBlocksForChunk(const FIoChunkId& Chunk, TFunction<bool(const FIoStoreTocCompressedBlockInfo&)>&& Callback) const;

	// Returns the .ucas file path and all partition(s) ({containername}_s1.ucas, {containername}_s2.ucas)
	CORE_API void GetContainerFilePaths(TArray<FString>& OutPaths);

private:
	FIoStoreReaderImpl* Impl;
};
