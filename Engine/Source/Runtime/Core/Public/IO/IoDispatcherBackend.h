// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IoDispatcher.h"
#include "Async/InheritedContext.h"
#include "Misc/Optional.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"

// TODO: Internal headers should not be included
#include "Runtime/Core/Internal/IO/IoContainers.h"
#include "Runtime/Core/Internal/IO/IoOffsetLength.h"

#define UE_IODISPATCHER_STATS_ENABLED (COUNTERSTRACE_ENABLED || CSV_PROFILER_STATS)

struct FIoContainerHeader;

/**
 * I/O request object.
 */
class FIoRequestImpl : private UE::FInheritedContextBase
{
public:
	/** Pointer to the next request that can be used freely by the I/O dispatcher backend(s). */
	FIoRequestImpl* NextRequest = nullptr;
	/** Custom data that can be used freely by the I/O dispatcher backend(s). */
	void* BackendData = nullptr;
	/** The chunk ID. */
	FIoChunkId ChunkId;
	/** Read options. */
	FIoReadOptions Options;
	/** I/O dispatcher priority (EIoDispatcherPriority). */
	TSAN_ATOMIC(int32) Priority = 0;

	FIoRequestImpl(class FIoRequestAllocator& InAllocator)
		: Allocator(InAllocator)
	{
		CaptureInheritedContext();
	}

	/** Returns whether the request has been cancelled. */
	bool IsCancelled() const
	{
		return LastBackendError == EIoErrorCode::Cancelled;
	}

	/** Returns whether the request failed. */
	bool IsFailed() const
	{
		return 
			LastBackendError != EIoErrorCode::Ok &&
			LastBackendError != EIoErrorCode::Cancelled;
	}

	/** Mark the request as failed (EIoErrorCode::ReadError). */
	void SetFailed()
	{
		if (!IsCancelled())
		{
			LastBackendError = EIoErrorCode::ReadError;
		}
	}

	/**
	 * Mark the request as failed, just like calling ::SetFailed only this version allows the caller to
	 * set the desired EIoErrorCode rather than accepting the default one.
	 */
	void SetLastBackendError(EIoErrorCode InError)
	{
		if (!IsCancelled())
		{
			LastBackendError = InError;
		}
	}

	/** Returns whether request has a valid buffer. */
	bool HasBuffer() const
	{
		return Buffer.IsSet();
	}

	/** Creates a new buffer for the request. */
	CORE_API void CreateBuffer(uint64 Size);

	/** Returns the internal buffer. */
	FIoBuffer& GetBuffer()
	{
		return Buffer.GetValue();
	}

	/** Sets a new buffer. */
	void SetResult(FIoBuffer InBuffer)
	{
		Buffer.Emplace(InBuffer);
	}

	uint64 GetStartTime() const
	{
#if UE_IODISPATCHER_STATS_ENABLED
		return StartTime;
#else
		return 0;
#endif
	}

private:
	friend class FIoDispatcherImpl;
	friend class FIoDispatcherInternal;
	friend class FIoRequest;
	friend class FIoBatch;
	friend class FIoRequestStats;

	void AddRef()
	{
		RefCount.IncrementExchange();
	}

	void ReleaseRef()
	{
		if (RefCount.DecrementExchange() == 1)
		{
			FreeRequest();
		}
	}

	void FreeRequest();

	FIoRequestAllocator& Allocator;
	struct IIoDispatcherBackend* Backend = nullptr;
	FIoBatchImpl* Batch = nullptr;
#if UE_IODISPATCHER_STATS_ENABLED
	uint64 StartTime = 0;
#endif
	TOptional<FIoBuffer> Buffer;
	FIoReadCallback Callback;
	TAtomic<uint32> RefCount{ 0 };
	TAtomic<EIoErrorCode> ErrorCode{ EIoErrorCode::Unknown };
	EIoErrorCode LastBackendError = EIoErrorCode::Ok;
};

namespace UE::Private
{

struct FIoRequestListTraits
{
	using ElementType = FIoRequestImpl;

	static FIoRequestImpl* GetNext(const FIoRequestImpl* Element)
	{
		return Element->NextRequest;
	}

	static void SetNext(FIoRequestImpl* Element, FIoRequestImpl* Next)
	{
		Element->NextRequest = Next;
	}
};

} // namespace UE::Private

using FIoRequestList = TIntrusiveList<UE::Private::FIoRequestListTraits>;

DECLARE_DELEGATE(FWakeUpIoDispatcherThreadDelegate);

/**
 * Context object used for signalling the I/O dispatcher.
 */ 
struct FIoDispatcherBackendContext
{
	/** Callback for signalling completed I/O requests. */
	FWakeUpIoDispatcherThreadDelegate WakeUpDispatcherThreadDelegate;
	/** Callback for signalling corrupted chunks. */
	FIoSignatureErrorDelegate SignatureErrorDelegate;
	/** Whether the I/O dispatcher is running in a separate thread. */
	bool bIsMultiThreaded;
};

/**
 * I/O dispatcher backend interface.
 *
 * The following methods are called from the I/O dispatcher thread:
 *		* ResolveIoRequests
 *		* CancelIoRequest
 *		* UpdatePriorityForIoRequest
 *		* GetCompletedRequests
 *
 *	All other methods can be called from any thread.
 */
struct IIoDispatcherBackend
{
	/**
	 * Called by the I/O dispatcher once initialized.
	 * @param Context	Context object used for signalling the I/O dispatcher.
	 */
	virtual void Initialize(TSharedRef<const FIoDispatcherBackendContext> Context) = 0;

	/** Called by the I/O dispatcher when shutting down. */
	virtual void Shutdown() {};

	/**
	 * Create asynchronous read requests for the list of I/O request(s).
	 * The request(s) can span multiple chunks.
	 *
	 * @param Requests			Request(s) to resolve.
	 * @param OutUnresolved		Unresolved request(s) returned to the I/O dispatcher.
	 */
	virtual void ResolveIoRequests(FIoRequestList Requests, FIoRequestList& OutUnresolved) = 0;

	/** 
	 * Returns all completed request(s) to the I/O dispatcher. Triggered
	 * by invoking the wake-up callback on the backend context object.
	 */
	virtual FIoRequestImpl* GetCompletedIoRequests() = 0;

	/** Cancel the I/O request. */
	virtual void CancelIoRequest(FIoRequestImpl* Request) = 0;

	/** Update the priority of the I/O request. */
	virtual void UpdatePriorityForIoRequest(FIoRequestImpl* Request) = 0;

	/** Returns whether the chunk exists. */
	virtual bool DoesChunkExist(const FIoChunkId& ChunkId) const = 0;

	/** Returns whether specified range of the chunk exists. */
	virtual bool DoesChunkExist(const FIoChunkId& ChunkId, const FIoOffsetAndLength& ChunkRange) const { return DoesChunkExist(ChunkId); }

	/** Returns the size of the chunk. */
	virtual TIoStatusOr<uint64> GetSizeForChunk(const FIoChunkId& ChunkId) const = 0;

	/** Returns the size of the chunk and the size of the requested range. */
	virtual TIoStatusOr<uint64> GetSizeForChunk(const FIoChunkId& ChunkId, const FIoOffsetAndLength& ChunkRange, uint64& OutAvailable) const
	{
		TIoStatusOr<uint64> ChunkSize = GetSizeForChunk(ChunkId);
		OutAvailable = ChunkSize.IsOk() ? ChunkSize.ValueOrDie() : 0;
		return ChunkSize;
	}

	/** Read the chunk as a memory mapped file. */
	virtual TIoStatusOr<FIoMappedRegion> OpenMapped(const FIoChunkId& ChunkId, const FIoReadOptions& Options) = 0;

	/** Returns the name of this backend, for logging purposes */
	virtual const TCHAR* GetName() const = 0;
};
