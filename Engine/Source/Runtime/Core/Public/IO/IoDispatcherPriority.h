// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include <stdint.h>

enum EIoDispatcherPriority : int32
{
	IoDispatcherPriority_Min = INT32_MIN,
	IoDispatcherPriority_Low = INT32_MIN / 2,
	IoDispatcherPriority_Medium = 0,
	IoDispatcherPriority_High = INT32_MAX / 2,
	IoDispatcherPriority_Max = INT32_MAX
};

