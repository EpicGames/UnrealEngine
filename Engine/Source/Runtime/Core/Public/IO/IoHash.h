// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "HAL/UnrealMemory.h"
#include "Hash/Blake3.h"
#include "Memory/MemoryFwd.h"
#include "Memory/MemoryView.h"
#include "Misc/AssertionMacros.h"
#include "String/BytesToHex.h"
#include "String/HexToBytes.h"

class FArchive;

/**
 * Stores a BLAKE3-160 hash, taken from the first 20 bytes of a BLAKE3-256 hash.
 *
 * The BLAKE3 hash function was selected for its high performance and its ability to parallelize.
 * Only the leading 160 bits of the 256-bit hash are used, to provide strong collision resistance
 * while minimizing the size of the hash.
 *
 * When the data to hash is not in a contiguous region of memory, use FIoHashBuilder to hash data
 * in blocks with FIoHashBuilder::Update(Block) followed by FIoHashBuilder::Finalize().
 */
struct alignas(uint32) FIoHash
{
public:
	using ByteArray = uint8[20];

	/** Construct a zero hash. */
	constexpr FIoHash() = default;

	/** Construct a hash from an array of 20 bytes. */
	inline explicit FIoHash(const ByteArray& Hash);

	/** Construct a hash from a BLAKE3-256 hash. */
	inline FIoHash(const FBlake3Hash& Hash);

	/** Construct a hash from a 40-character hex string. */
	inline explicit FIoHash(FAnsiStringView HexHash);
	inline explicit FIoHash(FWideStringView HexHash);
	inline explicit FIoHash(FUtf8StringView HexHash);

	/** Construct a hash from a view of 20 bytes. */
	inline static FIoHash FromView(FMemoryView Hash);

	/** Reset this to a zero hash. */
	inline void Reset() { *this = FIoHash(); }

	/** Returns whether this is a zero hash. */
	inline bool IsZero() const;

	/** Returns a reference to the raw byte array for the hash. */
	inline ByteArray& GetBytes() { return Hash; }
	inline const ByteArray& GetBytes() const { return Hash; }

	/** Calculate the hash of the buffer. */
	[[nodiscard]] inline static FIoHash HashBuffer(FMemoryView View);
	[[nodiscard]] inline static FIoHash HashBuffer(const void* Data, uint64 Size);
	[[nodiscard]] CORE_API static FIoHash HashBuffer(const FCompositeBuffer& Buffer);

	/** A zero hash. */
	static const FIoHash Zero;

private:
	ByteArray Hash{};

	friend inline bool operator==(const FIoHash& A, const FIoHash& B)
	{
		return FMemory::Memcmp(A.GetBytes(), B.GetBytes(), sizeof(decltype(A.GetBytes()))) == 0;
	}

	friend inline bool operator!=(const FIoHash& A, const FIoHash& B)
	{
		return FMemory::Memcmp(A.GetBytes(), B.GetBytes(), sizeof(decltype(A.GetBytes()))) != 0;
	}

	friend inline bool operator<(const FIoHash& A, const FIoHash& B)
	{
		return FMemory::Memcmp(A.GetBytes(), B.GetBytes(), sizeof(decltype(A.GetBytes()))) < 0;
	}

	friend inline uint32 GetTypeHash(const FIoHash& Value)
	{
		return *reinterpret_cast<const uint32*>(Value.GetBytes());
	}

	friend CORE_API FArchive& operator<<(FArchive& Ar, FIoHash& InHash);
};

inline const FIoHash FIoHash::Zero;

inline FIoHash::FIoHash(const ByteArray& InHash)
{
	FMemory::Memcpy(Hash, InHash, sizeof(ByteArray));
}

inline FIoHash::FIoHash(const FBlake3Hash& InHash)
{
	static_assert(sizeof(ByteArray) <= sizeof(decltype(InHash.GetBytes())), "Reading too many bytes from source.");
	FMemory::Memcpy(Hash, InHash.GetBytes(), sizeof(ByteArray));
}

inline FIoHash::FIoHash(const FAnsiStringView HexHash)
{
	check(HexHash.Len() == sizeof(ByteArray) * 2);
	UE::String::HexToBytes(HexHash, Hash);
}

inline FIoHash::FIoHash(const FWideStringView HexHash)
{
	check(HexHash.Len() == sizeof(ByteArray) * 2);
	UE::String::HexToBytes(HexHash, Hash);
}

inline FIoHash::FIoHash(const FUtf8StringView HexHash)
{
	check(HexHash.Len() == sizeof(ByteArray) * 2);
	UE::String::HexToBytes(HexHash, Hash);
}

inline FIoHash FIoHash::FromView(const FMemoryView InHash)
{
	checkf(InHash.GetSize() == sizeof(ByteArray),
		TEXT("FIoHash cannot be constructed from a view of %" UINT64_FMT " bytes."), InHash.GetSize());
	FIoHash NewHash;
	FMemory::Memcpy(NewHash.Hash, InHash.GetData(), sizeof(ByteArray));
	return NewHash;
}

inline bool FIoHash::IsZero() const
{
	using UInt32Array = uint32[5];
	static_assert(sizeof(UInt32Array) == sizeof(ByteArray), "Invalid size for UInt32Array");
	for (uint32 Value : reinterpret_cast<const UInt32Array&>(Hash))
	{
		if (Value != 0)
		{
			return false;
		}
	}
	return true;
}

inline FIoHash FIoHash::HashBuffer(FMemoryView View)
{
	return FBlake3::HashBuffer(View);
}

inline FIoHash FIoHash::HashBuffer(const void* Data, uint64 Size)
{
	return FBlake3::HashBuffer(Data, Size);
}

template <typename CharType>
inline TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const FIoHash& Hash)
{
	UE::String::BytesToHexLower(Hash.GetBytes(), Builder);
	return Builder;
}

/** Construct a hash from a 40-character hex string. */
inline void LexFromString(FIoHash& OutHash, const TCHAR* Buffer)
{
	OutHash = FIoHash(Buffer);
}

/** Construct a hash from a 40-character hex string. */
template <typename CharType>
inline bool TryLexFromString(FIoHash& OutHash, TStringView<CharType> String)
{
	if (String.Len() == sizeof(FIoHash) * 2)
	{
		OutHash = FIoHash(String);
		return true;
	}
	return false;
}

/** Convert a hash to a 40-character hex string. */
[[nodiscard]] CORE_API FString LexToString(const FIoHash& Hash);

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Calculates a BLAKE3-160 hash. */
class FIoHashBuilder : public FBlake3
{
public:
	/**
	 * Finalize the hash of the input data.
	 *
	 * May be called any number of times, and more input may be added after.
	 */
	[[nodiscard]] inline FIoHash Finalize() const { return FBlake3::Finalize(); }

	/** Calculate the hash of the buffer. */
	[[nodiscard]] inline static FIoHash HashBuffer(FMemoryView View) { return FBlake3::HashBuffer(View); }
	[[nodiscard]] inline static FIoHash HashBuffer(const void* Data, uint64 Size) { return FBlake3::HashBuffer(Data, Size); }
	[[nodiscard]] CORE_API static FIoHash HashBuffer(const FCompositeBuffer& Buffer);
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Containers/UnrealString.h"
#include "Serialization/Archive.h"
#endif
