// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"
#include "Containers/SharedString.h"
#include "Containers/UnrealString.h"
#include "Misc/StringBuilder.h"

template <typename CharType> class TStringBuilderBase;


///////////////////////////////////////////////////////////////////////////////

/**
 * I/O error code.
 * **IMPORTANT** This enum must be kept in sync with error declarations in IoStatusError.h
 */
enum class EIoErrorCode
{
	Ok,
	Unknown,
	InvalidCode,
	Cancelled,
	FileOpenFailed,
	FileNotOpen,
	ReadError,
	WriteError,
	NotFound,
	CorruptToc,
	UnknownChunkID,
	InvalidParameter,
	SignatureError,
	InvalidEncryptionKey,
	CompressionError,
	PendingFork,
	PendingEncryptionKey,
	Disabled,
	NotInstalled,
	PendingHostGroup,
	Timeout,
	DeleteError,
	OutOfDiskSpace,
	FileSeekFailed,
	FileFlushFailed,
	FileMoveFailed,
	FileCloseFailed,
	FileCorrupt,
	HttpReadError,
	FileHandleOpen,
	PendingDelete,
	Last
};

/** Get I/O error code description. */
CORE_API const TCHAR* GetIoErrorText(EIoErrorCode ErrorCode);

///////////////////////////////////////////////////////////////////////////////

/**
 * I/O status with error code, error message and an optional system error code.
 */
class FIoStatus
{
public:
	CORE_API			FIoStatus();
	CORE_API			~FIoStatus();

	CORE_API			FIoStatus(EIoErrorCode ErrorCode, const FStringView& ErrorMessage);
	CORE_API			FIoStatus(EIoErrorCode ErrorCode, uint32 SystemErrorCode, const FStringView& ErrorMessage);
	CORE_API			FIoStatus(EIoErrorCode ErrorCode, uint32 SystemErrorCode);
	CORE_API			FIoStatus(EIoErrorCode ErrorCode);
	CORE_API FIoStatus&	operator=(const FIoStatus& Other);
	CORE_API FIoStatus&	operator=(const EIoErrorCode ErrorCode);

	CORE_API bool		operator==(const FIoStatus& Other) const;
			 bool		operator!=(const FIoStatus& Other) const { return !operator==(Other); }

	inline bool			IsOk() const { return ErrorCode == EIoErrorCode::Ok; }
	inline bool			IsCompleted() const { return ErrorCode != EIoErrorCode::Unknown; }
	inline EIoErrorCode	GetErrorCode() const { return ErrorCode; }
	inline uint32		GetSystemErrorCode() const { return SystemErrorCode; }
	const TCHAR*		GetErrorMessage() const { return ErrorMessage.IsEmpty() ? nullptr : *ErrorMessage; }
	CORE_API FString	ToString() const;

	CORE_API friend void SerializeForLog(FCbWriter& Writer, const FIoStatus& Value);

	CORE_API static const FIoStatus Ok;
	CORE_API static const FIoStatus Unknown;
	CORE_API static const FIoStatus Invalid;

private:
	UE::FSharedString	ErrorMessage;
	EIoErrorCode		ErrorCode = EIoErrorCode::Ok;
	uint32				SystemErrorCode = 0;

	friend class FIoStatusBuilder;
};

/**
 * Optional I/O result or error status.
 */
template<typename T>
class TIoStatusOr
{
	template<typename U> friend class TIoStatusOr;

public:
	TIoStatusOr() : StatusValue(FIoStatus::Unknown) {}
	TIoStatusOr(const TIoStatusOr& Other);
	TIoStatusOr(TIoStatusOr&& Other);

	TIoStatusOr(FIoStatus InStatus);
	TIoStatusOr(const T& InValue);
	TIoStatusOr(T&& InValue);

	~TIoStatusOr();

	template <typename... ArgTypes>
	explicit TIoStatusOr(ArgTypes&&... Args);

	template<typename U>
	TIoStatusOr(const TIoStatusOr<U>& Other);

	TIoStatusOr<T>& operator=(const TIoStatusOr<T>& Other);
	TIoStatusOr<T>& operator=(TIoStatusOr<T>&& Other);
	TIoStatusOr<T>& operator=(const FIoStatus& OtherStatus);
	TIoStatusOr<T>& operator=(const T& OtherValue);
	TIoStatusOr<T>& operator=(T&& OtherValue);

	template<typename U>
	TIoStatusOr<T>& operator=(const TIoStatusOr<U>& Other);

	const FIoStatus&	Status() const;
	bool				IsOk() const;

	const T&			ValueOrDie() const;
	T&					ValueOrDie();
	T					ConsumeValueOrDie();

	void				Reset();

private:
	FIoStatus				StatusValue;
	TTypeCompatibleBytes<T>	Value;
};

///////////////////////////////////////////////////////////////////////////////

/**
 * Helper to make it easier to generate meaningful error messages.
 */
class FIoStatusBuilder
{
protected:
	EIoErrorCode		StatusCode;
	uint32				SystemErrorCode;
	TStringBuilder<256> Message;
public:
	CORE_API explicit	FIoStatusBuilder(EIoErrorCode StatusCode);
	CORE_API explicit	FIoStatusBuilder(EIoErrorCode StatusCode, uint32 SystemErrorCode);
	CORE_API			~FIoStatusBuilder();

	CORE_API			operator FIoStatus();

	template<typename T>
	operator TIoStatusOr<T>()
	{
		return FIoStatus(StatusCode, SystemErrorCode, Message);
	}

	CORE_API FIoStatusBuilder& operator<<(FStringView String);
};

CORE_API void StatusOrCrash(const FIoStatus& Status);

template<typename T>
void TIoStatusOr<T>::Reset()
{
	EIoErrorCode ErrorCode = StatusValue.GetErrorCode();
	StatusValue = EIoErrorCode::Unknown;

	if (ErrorCode == EIoErrorCode::Ok)
	{
		((T*)&Value)->~T();
	}
}

template<typename T>
const T& TIoStatusOr<T>::ValueOrDie() const
{
	if (!StatusValue.IsOk())
	{
		StatusOrCrash(StatusValue);
	}

	return *Value.GetTypedPtr();
}

template<typename T>
inline T& TIoStatusOr<T>::ValueOrDie()
{
	if (!StatusValue.IsOk())
	{
		StatusOrCrash(StatusValue);
	}

	return *Value.GetTypedPtr();
}

template<typename T>
T TIoStatusOr<T>::ConsumeValueOrDie()
{
	if (!StatusValue.IsOk())
	{
		StatusOrCrash(StatusValue);
	}

	StatusValue = FIoStatus::Unknown;

	return MoveTemp(*Value.GetTypedPtr());
}

template<typename T>
TIoStatusOr<T>::TIoStatusOr(const TIoStatusOr& Other)
{
	StatusValue = Other.StatusValue;
	if (StatusValue.IsOk())
	{
		new(&Value) T(*(const T*)&Other.Value);
	}
}

template<typename T>
TIoStatusOr<T>::TIoStatusOr(TIoStatusOr&& Other)
{
	StatusValue = Other.StatusValue;
	if (StatusValue.IsOk())
	{
		new(&Value) T(MoveTempIfPossible(*(T*)&Other.Value));
		Other.StatusValue = EIoErrorCode::Unknown;
	}
}

template<typename T>
TIoStatusOr<T>::TIoStatusOr(FIoStatus InStatus)
{
	check(!InStatus.IsOk());
	StatusValue = InStatus;
}

template<typename T>
TIoStatusOr<T>::TIoStatusOr(const T& InValue)
{
	StatusValue = FIoStatus::Ok;
	new(&Value) T(InValue);
}

template<typename T>
TIoStatusOr<T>::TIoStatusOr(T&& InValue)
{
	StatusValue = FIoStatus::Ok;
	new(&Value) T(MoveTempIfPossible(InValue));
}

template <typename T>
template <typename... ArgTypes>
TIoStatusOr<T>::TIoStatusOr(ArgTypes&&... Args)
{
	StatusValue = FIoStatus::Ok;
	new(&Value) T(Forward<ArgTypes>(Args)...);
}

template<typename T>
TIoStatusOr<T>::~TIoStatusOr()
{
	Reset();
}

template<typename T>
bool TIoStatusOr<T>::IsOk() const
{
	return StatusValue.IsOk();
}

template<typename T>
const FIoStatus& TIoStatusOr<T>::Status() const
{
	return StatusValue;
}

template<typename T>
TIoStatusOr<T>&
TIoStatusOr<T>::operator=(const TIoStatusOr<T>& Other)
{
	if (&Other != this)
	{
		Reset();

		if (Other.StatusValue.IsOk())
		{
			new(&Value) T(*(const T*)&Other.Value);
			StatusValue = EIoErrorCode::Ok;
		}
		else
		{
			StatusValue = Other.StatusValue;
		}
	}

	return *this;
}

template<typename T>
TIoStatusOr<T>&
TIoStatusOr<T>::operator=(TIoStatusOr<T>&& Other)
{
	if (&Other != this)
	{
		Reset();
 
		if (Other.StatusValue.IsOk())
		{
			new(&Value) T(MoveTempIfPossible(*(T*)&Other.Value));
			Other.StatusValue = EIoErrorCode::Unknown;
			StatusValue = EIoErrorCode::Ok;
		}
		else
		{
			StatusValue = Other.StatusValue;
		}
	}

	return *this;
}

template<typename T>
TIoStatusOr<T>&
TIoStatusOr<T>::operator=(const FIoStatus& OtherStatus)
{
	check(!OtherStatus.IsOk());

	Reset();
	StatusValue = OtherStatus;

	return *this;
}

template<typename T>
TIoStatusOr<T>&
TIoStatusOr<T>::operator=(const T& OtherValue)
{
	if (&OtherValue != (T*)&Value)
	{
		Reset();
		
		new(&Value) T(OtherValue);
		StatusValue = EIoErrorCode::Ok;
	}

	return *this;
}

template<typename T>
TIoStatusOr<T>&
TIoStatusOr<T>::operator=(T&& OtherValue)
{
	if (&OtherValue != (T*)&Value)
	{
		Reset();
		
		new(&Value) T(MoveTempIfPossible(OtherValue));
		StatusValue = EIoErrorCode::Ok;
	}

	return *this;
}

template<typename T>
template<typename U>
TIoStatusOr<T>::TIoStatusOr(const TIoStatusOr<U>& Other)
:	StatusValue(Other.StatusValue)
{
	if (StatusValue.IsOk())
	{
		new(&Value) T(*(const U*)&Other.Value);
	}
}

template<typename T>
template<typename U>
TIoStatusOr<T>& TIoStatusOr<T>::operator=(const TIoStatusOr<U>& Other)
{
	Reset();

	if (Other.StatusValue.IsOk())
	{
		new(&Value) T(*(const U*)&Other.Value);
		StatusValue = EIoErrorCode::Ok;
	}
	else
	{
		StatusValue = Other.StatusValue;
	}

	return *this;
}
