// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/StringFwd.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"

class FArchive;
class FStructuredArchiveSlot;

#ifndef WITH_PACKAGEID_NAME_MAP
#define WITH_PACKAGEID_NAME_MAP WITH_EDITOR
#endif

class FPackageId
{
	static constexpr uint64 InvalidId = 0;
	uint64 Id = InvalidId;

	inline explicit FPackageId(uint64 InId): Id(InId) {}

public:
	constexpr FPackageId() = default;

	CORE_API static FPackageId FromName(const FName& Name);

	static FPackageId FromValue(const uint64 Value)
	{
		return FPackageId(Value);
	}

	inline bool IsValid() const
	{
		return Id != InvalidId;
	}

	inline uint64 Value() const
	{
		return Id;
	}

	inline bool operator<(FPackageId Other) const
	{
		return Id < Other.Id;
	}

	inline bool operator==(FPackageId Other) const
	{
		return Id == Other.Id;
	}
	
	inline bool operator!=(FPackageId Other) const
	{
		return Id != Other.Id;
	}

	inline friend uint32 GetTypeHash(const FPackageId& In)
	{
		return uint32(In.Id);
	}

	CORE_API friend FArchive& operator<<(FArchive& Ar, FPackageId& Value);

	CORE_API friend void operator<<(FStructuredArchiveSlot Slot, FPackageId& Value);

	CORE_API friend void SerializeForLog(FCbWriter& Writer, const FPackageId& Value);

#if WITH_PACKAGEID_NAME_MAP
	CORE_API FName GetName() const;
#endif
};

CORE_API FString LexToString(const FPackageId& PackageId);
CORE_API void LexFromString(FPackageId& Out, FStringView String);
CORE_API void LexFromString(FPackageId& Out, const TCHAR* String);

template <typename CharType>
TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const FPackageId& PackageId)
{
	Builder.Appendf(CHARTEXT(CharType, "0x%llX"), PackageId.Value());
#if WITH_PACKAGEID_NAME_MAP
	Builder << " (" << PackageId.GetName() << ")";
#endif // WITH_PACKAGEID_NAME_MAP
	return Builder;
}
