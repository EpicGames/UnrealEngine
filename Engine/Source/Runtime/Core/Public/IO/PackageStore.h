// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Deprecated

#include "CoreTypes.h"

UE_DEPRECATED_HEADER(5.6, "PackageStore has moved from Core to CoreUObject. #include 'Serialization/PackageStore.h' instead of 'IO/PackageStore.h'. Attempting to automatically include the correct one, but if the include fails, add a dependency on CoreUObject in your .Build.cs file, and update your include line.")

#include "Serialization/PackageStore.h"
