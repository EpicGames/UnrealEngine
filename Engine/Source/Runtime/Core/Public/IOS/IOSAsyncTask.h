// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Apple/AppleAsyncTask.h"

UE_DEPRECATED(5.7, "Use the more general FAppleAsyncTask instead")
@interface FIOSAsyncTask : FAppleAsyncTask
{
}
@end
