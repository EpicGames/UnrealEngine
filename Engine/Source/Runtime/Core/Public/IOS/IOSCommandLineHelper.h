// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Apple/AppleStringUtils.h"
#include "CoreMinimal.h"
#include "Containers/StringView.h"
#include "Misc/CommandLine.h"

#define IOS_MAX_PATH 1024
#define CMD_LINE_MAX 16384

extern APPLICATIONCORE_API FString GSavedCommandLine;

class FIOSCommandLineHelper
{
	public:
		/**
		 * Merge the given commandline with GSavedCommandLinePortion, which may start with ?
		 * options that need to come after first token
		 */

		static void MergeCommandlineWithSaved(TCHAR CommandLine[16384])
		{
			// the saved commandline may be in the format ?opt?opt -opt -opt, so we need to insert it 
			// after the first token on the commandline unless the first token starts with a -, in which 
			// case use it at the start of the command line
			if (CommandLine[0] == '-' || CommandLine[0] == 0)
			{
				// handle the easy - case, just use the saved command line part as the start, in case it
				// started with a ?
				FString CombinedCommandLine = GSavedCommandLine + CommandLine;
				FCString::Strncpy(CommandLine, *CombinedCommandLine, CMD_LINE_MAX);
			}
			else
			{
				// otherwise, we need to get the first token from the command line and insert after
				TCHAR* Space = FCString::Strchr(CommandLine, ' ');
				if (Space == NULL)
				{
					// if there is only one token (no spaces), just append us after it
					FCString::StrncatTruncateDest(CommandLine, CMD_LINE_MAX, *GSavedCommandLine);
				}
				else
				{
					// save off what's after the space (include the space for pasting later)
					FString AfterSpace(Space);
					// copy the save part where the space was
					FCString::Strncpy(Space, *GSavedCommandLine, CMD_LINE_MAX - (Space - CommandLine));
					// now put back the 2nd and so on token
					FCString::StrncatTruncateDest(CommandLine, CMD_LINE_MAX, *AfterSpace);
				}
			}
		}

		static bool TryReadCommandLineFile(const FString& CommandLineFilePath)
		{
			bool bHasFile = false;
			
			// initialize the commandline
			// read in the command line text file (coming from UnrealFrontend) if it exists
			FILE* CommandLineFile = fopen(TCHAR_TO_UTF8(*CommandLineFilePath), "r");
			if(CommandLineFile)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Checking for command line in %s... FOUND!") LINE_TERMINATOR, *CommandLineFilePath);
				UTF8CHAR CommandLine[CMD_LINE_MAX];
				if (fgets((char*)CommandLine, CMD_LINE_MAX, CommandLineFile))
				{
					// chop off trailing spaces
					int32 CommandLineLength = FUtf8StringView{CommandLine}.TrimEnd().Len();
					CommandLine[CommandLineLength] = '\0';

					FCommandLine::Append(StringCast<TCHAR>(CommandLine, CommandLineLength + 1).Get());
				}
				fclose(CommandLineFile);
				bHasFile = true;
			}
			else
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Checking for command line in %s... NOT FOUND!") LINE_TERMINATOR, *CommandLineFilePath);
			}
			
			return bHasFile;
		}

		static void InitCommandArgs(FString AdditionalCommandArgs)
		{
			// initialize the commandline
			FCommandLine::Set(TEXT(""));

			// command line text file included in the bundle
			FString BundleCommandLineFilePath = FString([[NSBundle mainBundle] bundlePath]) + TEXT("/uecommandline.txt");

			// command line text file pushed to the documents folder
			FString DocumentsCommandLineFilePath = FString([NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/uecommandline.txt");
			
#if UE_BUILD_SHIPPING
			bool bIsTestFlightOrInternal = [[[[NSBundle mainBundle] appStoreReceiptURL] lastPathComponent] isEqualToString: @"sandboxReceipt"];
#else
			bool bIsTestFlightOrInternal = YES;
#endif

#if ENABLE_PGO_PROFILE
			// We need to be able to override command-line for PGO profiling builds.
			bIsTestFlightOrInternal = YES;
#endif

			bool bFoundDocumentsCommandLine = bIsTestFlightOrInternal && TryReadCommandLineFile(DocumentsCommandLineFilePath);
			if (!bFoundDocumentsCommandLine)
			{
				TryReadCommandLineFile(BundleCommandLineFilePath);
			}

			FString ReplacementCL, AppendCL;
			bool bHasReplacementCL = FPlatformMisc::GetStoredValue(TEXT(""), TEXT("IOSCommandLine"), TEXT("ReplacementCL"), ReplacementCL);
			bool bHasAppendCL = FPlatformMisc::GetStoredValue(TEXT(""), TEXT("IOSCommandLine"), TEXT("AppendCL"), AppendCL);
			
			if (bHasReplacementCL)
			{
				FCommandLine::Set(*ReplacementCL);
			}
			
			if (bHasAppendCL)
			{
				FCommandLine::Append(TEXT(" "));
				FCommandLine::Append(*AppendCL);
			}

			if (!AdditionalCommandArgs.IsEmpty() && !FChar::IsWhitespace(AdditionalCommandArgs[0]))
			{
				FCommandLine::Append(TEXT(" "));
				FCommandLine::Append(*AdditionalCommandArgs);
			}

			// now merge the GSavedCommandLine with the rest
			FCommandLine::Append(TEXT(" "));
			FCommandLine::Append(*GSavedCommandLine);

			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Combined iOS Commandline: %s") LINE_TERMINATOR, FCommandLine::Get());
		}
};

