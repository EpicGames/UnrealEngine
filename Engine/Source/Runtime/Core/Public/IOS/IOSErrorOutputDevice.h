// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/OutputDevice.h"
#include "Misc/OutputDeviceError.h"

class CORE_API FIOSErrorOutputDevice : public FOutputDeviceError
{
public:
	/** Constructor, initializing member variables */
	FIOSErrorOutputDevice();

	/**
	 * Serializes the passed in data unless the current event is suppressed.
	 *
	 * @param	Data	Text to log
	 * @param	Event	Event name used for suppression purposes
	 */
	virtual void Serialize( const TCHAR* Msg, ELogVerbosity::Type Verbosity, const class FName& Category ) override;

	/**
	 * Error handling function that is being called from within the system wide global
	 * error handler, e.g. using structured exception handling on the PC.
	 */
	void HandleError();

private:
	int32		ErrorPos;
};
