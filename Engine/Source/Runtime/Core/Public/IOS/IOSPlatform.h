// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	IOSPlatform.h: Setup for the iOS platform
==================================================================================*/

#pragma once

#include "Apple/ApplePlatform.h"
#include "Availability.h"

/**
* iOS specific types
**/
struct FIOSPlatformTypes : public FGenericPlatformTypes
{
	typedef size_t				SIZE_T;
	typedef decltype(NULL)		TYPE_OF_NULL;
	typedef char16_t			WIDECHAR;
	typedef WIDECHAR			TCHAR;
};

typedef FIOSPlatformTypes FPlatformTypes;

// Base defines, must define these for the platform, there are no defaults
#define PLATFORM_DESKTOP								0

// Base defines, defaults are commented out
#define PLATFORM_TCHAR_IS_CHAR16						1
#define PLATFORM_MAX_FILEPATH_LENGTH_DEPRECATED			IOS_MAX_PATH
#define PLATFORM_BUILTIN_VERTEX_HALF_FLOAT				0
#define PLATFORM_SUPPORTS_MULTIPLE_NATIVE_WINDOWS		0
#define PLATFORM_ALLOW_NULL_RHI							1
#define PLATFORM_ENABLE_VECTORINTRINSICS_NEON			1
#define PLATFORM_SUPPORTS_EARLY_MOVIE_PLAYBACK			1 // movies will start before engine is initalized
#define PLATFORM_USE_FULL_TASK_GRAPH					0 // @todo platplug: not platplug, but should investigate soon anyway

#define PLATFORM_NUM_AUDIODECOMPRESSION_PRECACHE_BUFFERS 0
#if PLATFORM_TVOS
#	define PLATFORM_USES_GLES							0
#	define PLATFORM_HAS_TOUCH_MAIN_SCREEN				0
#	define	PLATFORM_SUPPORTS_OPUS_CODEC				0
#	define PLATFORM_SUPPORTS_VORBIS_CODEC				0
#else
#	define PLATFORM_USES_GLES							1
#	define PLATFORM_HAS_TOUCH_MAIN_SCREEN				1
#endif
#define PLATFORM_UI_HAS_MOBILE_SCROLLBARS				1
#define PLATFORM_UI_NEEDS_TOOLTIPS						0
#define PLATFORM_UI_NEEDS_FOCUS_OUTLINES				0

#define PLATFORM_NEEDS_RHIRESOURCELIST					0
#define PLATFORM_SUPPORTS_GEOMETRY_SHADERS				0

#ifndef PLATFORM_SUPPORTS_BINDLESS_RENDERING
#define PLATFORM_SUPPORTS_BINDLESS_RENDERING			0
#endif

#define PLATFORM_RETURN_ADDRESS_FOR_CALLSTACKTRACING    PLATFORM_RETURN_ADDRESS

#define PLATFORM_GLOBAL_LOG_CATEGORY					LogIOS

//mallocpoison not safe with aligned ansi allocator.  returns the larger unaligned size during Free() which causes writes off the end of the allocation.
#define UE_USE_MALLOC_FILL_BYTES						0 

#define IOS_MAX_PATH									1024

#if UE_BUILD_DEBUG || UE_DISABLE_FORCE_INLINE
#	define FORCEINLINE									inline 									/* Don't force code to be inline */
#else
#	define FORCEINLINE									inline __attribute__ ((always_inline))	/* Force code to be inline */
#endif

// Strings.
#define LINE_TERMINATOR TEXT("\n")
#define LINE_TERMINATOR_ANSI "\n"

static_assert(__IPHONE_OS_VERSION_MAX_ALLOWED >= 13000, "Unreal requires Xcode 11 or later to build"); 
