// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Apple/ApplePlatformAffinity.h"

class CORE_API FIOSPlatformAffinity : public FApplePlatformAffinity
{
public:
	static const uint64 GetMainGameMask();
	static const uint64 GetRenderingThreadMask();
	static const uint64 GetRTHeartBeatMask();
	static const uint64 GetPoolThreadMask();
	static const uint64 GetTaskGraphThreadMask();
	static const uint64 GetNoAffinityMask();
};

typedef FIOSPlatformAffinity FPlatformAffinityBase;
