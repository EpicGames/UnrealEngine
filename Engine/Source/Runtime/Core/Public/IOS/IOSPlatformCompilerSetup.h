// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	IOSPlatformCompilerSetup.h: pragmas, version checks and other things for the iOS compiler
==============================================================================================*/

#pragma once

/**
 * We require at least Xcode 9.4 to compile
 */
static_assert((__clang_major__ > 9) || (__clang_major__ == 9 && __clang_minor__ >= 1), "Xcode 9.4 or newer is required to compile on Mac. Please make sure it's installed and selected as default using xcode-select tool.");
