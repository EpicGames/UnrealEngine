// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Apple/ApplePlatformCrashContext.h"
#include "Misc/Guid.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#import <Foundation/NSProcessInfo.h>
#endif

//Forward Declarations
class FIOSMallocCrashHandler;
#if !PLATFORM_TVOS
@class PLCrashReporter;
#endif

/*------------------------------------------------------------------------------
 FIOSApplicationInfo - class to contain all state for crash reporting that is unsafe to acquire in a signal.
 ------------------------------------------------------------------------------*/

/**
 * Information that cannot be obtained during a signal-handler is initialised here.
 * This ensures that we only call safe functions within the crash reporting handler.
 */
struct FIOSApplicationInfo
{
	void Init();
    
	~FIOSApplicationInfo();
    
	static FGuid RunGUID();
    
	static FString TemporaryCrashReportFolder();
	static FString TemporaryCrashReportName();

    
    bool bIsSandboxed;
    int32 NumCores;
    char AppNameUTF8[PATH_MAX+1];
    char AppLogPath[PATH_MAX+1];
    char CrashReportPath[PATH_MAX+1];
    char PLCrashReportPath[PATH_MAX+1];
    char OSVersionUTF8[PATH_MAX+1];
    char MachineName[PATH_MAX+1];
    char MachineCPUString[PATH_MAX+1];
    FString AppPath;
    FString AppName;
    FString AppBundleID;
    FString OSVersion;
    FString OSBuild;
    FString MachineUUID;
    FString MachineModel;
    FString BiosRelease;
    FString BiosRevision;
    FString BiosUUID;
    FString ParentProcess;
    FString LCID;
    FString CommandLine;
    FString BranchBaseDir;
    FString PrimaryGPU;
    FString ExecutableName;
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
	UE_DEPRECATED(5.8, "The OSXVersion property is no longer available. Use the already parsed version in string format")
    NSOperatingSystemVersion OSXVersion;
#endif// UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
	
    FGuid RunUUID;
    FString XcodePath;
#if !PLATFORM_TVOS
    static PLCrashReporter* CrashReporter;
#endif
    static FIOSMallocCrashHandler* CrashMalloc;
};

struct CORE_API FIOSCrashContext : public FApplePlatformCrashContext
{
	/** Constructor */
	FIOSCrashContext(ECrashContextType InType, const TCHAR* InErrorMessage);

	/** Copies the PLCrashReporter minidump */
	void CopyMinidump(char const* OutputPath, char const* InputPath) const;

	/** Generates the ensure/crash info into the given folder */
	void GenerateInfoInFolder(char const* const InfoFolder, bool bIsEnsure = false) const;
	
	/** Generates information for crash reporter */
	void GenerateCrashInfo() const;
	
	/** Generates information for ensures sent via the CrashReporter */
	void GenerateEnsureInfo() const;
	
	/** Creates the crash folder */
	FString CreateCrashFolder() const;
    
    static void SetGenerateScreenshotFunc(const TFunction<void(char const* const InfoFolder)>& InGenerateScreenshotFunc);
	
private:
	/** Serializes platform specific properties to the buffer. */
	virtual void AddPlatformSpecificProperties() const;

    static TFunction<void(char const* const InfoFolder)> GenerateScreenshotFunc;
};

//Single global FIOSApplicationInfo to store our App's information
extern FIOSApplicationInfo GIOSAppInfo;

typedef FIOSCrashContext FPlatformCrashContext;
