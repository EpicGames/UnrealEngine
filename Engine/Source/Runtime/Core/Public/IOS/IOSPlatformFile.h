// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Apple/ApplePlatformFile.h"
#include "Containers/UnrealString.h"

class IFileHandle;
class FApplePlatformFile;


/**
 * iOS File I/O implementation
**/
class CORE_API FIOSPlatformFile : public FApplePlatformFile
{
protected:
	virtual FString NormalizeFilename(const TCHAR* Filename);
	virtual FString NormalizeDirectory(const TCHAR* Directory);

public:
	FIOSPlatformFile();

	//virtual bool Initialize(IPlatformFile* Inner, const TCHAR* CommandLineParam) override;

	virtual bool FileExists(const TCHAR* Filename) override;
	virtual int64 FileSize(const TCHAR* Filename) override;
	virtual bool DeleteFile(const TCHAR* Filename) override;
	virtual bool IsReadOnly(const TCHAR* Filename) override;
	virtual bool MoveFile(const TCHAR* To, const TCHAR* From) override;
	virtual bool SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override;

	virtual FDateTime GetTimeStamp(const TCHAR* Filename) override;
	virtual void SetTimeStamp(const TCHAR* Filename, const FDateTime DateTime) override;
	virtual FDateTime GetAccessTimeStamp(const TCHAR* Filename) override;
	virtual FString GetFilenameOnDisk(const TCHAR* Filename) override;
    
    virtual FString ConvertToAbsolutePathForExternalAppForRead( const TCHAR* Filename ) override;
    virtual FString ConvertToAbsolutePathForExternalAppForWrite( const TCHAR* Filename ) override;

	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override;

	virtual IFileHandle* OpenRead(const TCHAR* Filename, bool bAllowWrite = false) override;
	virtual IFileHandle* OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override;

	virtual bool DirectoryExists(const TCHAR* Directory) override;
	virtual bool CreateDirectory(const TCHAR* Directory) override;
	virtual bool DeleteDirectory(const TCHAR* Directory) override;

	virtual bool IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor) override;
	virtual bool IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor) override;

	virtual bool DoesCreatePublicFiles() override;
	virtual void SetCreatePublicFiles(bool bCreatePublicFilesIn) override;
	
	// If app has a Shared File Container via the AppGroups capability, return the
	// path or an empty string if it's not found
	static FString GetSharedContainerPath(const TCHAR* AppGroupID);
	
protected:
	virtual FString ConvertToPlatformPath(const FString& Filename, bool bForWrite, bool bIsPublicWrite) override;
	virtual int32 OpenFileForMemMap(const FString& NormalizedFilename, EOpenMappedFlags OpenOptions, FString& OutFinalPath) override;

private:
	bool IterateDirectoryCommon(const TCHAR* Directory, const TFunctionRef<bool(struct dirent*)>& Visitor);

	FString ReadDirectory;
	FString WriteDirectory;
	
	bool bCreatePublicFiles;
};
