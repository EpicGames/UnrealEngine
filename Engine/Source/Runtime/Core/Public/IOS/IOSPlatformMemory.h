// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	IOSPlatformMemory.h: IOS platform memory functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformMemory.h"
#include "Apple/ApplePlatformMemory.h"

/**
* IOS implementation of the memory OS functions
**/
struct CORE_API FIOSPlatformMemory : public FApplePlatformMemory
{
    // added this for now because Crashlytics doesn't properly break up different callstacks all ending in UE_LOG(LogXXX, Fatal, ...)
    [[noreturn]] static void OnOutOfMemory(uint64 Size, uint32 Alignment);
};

typedef FIOSPlatformMemory FPlatformMemory;
