// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	IOSPlatformMisc.h: iOS platform misc functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformMisc.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "IOS/IOSSystemIncludes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#ifdef __OBJC__
@class Protocol;
#endif

#include "Apple/ApplePlatformMisc.h"

template <typename FuncType>
class TFunction;

/**
* iOS implementation of the misc OS functions
**/
struct CORE_API FIOSPlatformMisc : public FApplePlatformMisc
{
    static void PlatformPreInit();
	static void PlatformInit();
	static void PlatformTearDown();
    static void PlatformHandleSplashScreen(bool ShowSplashScreen = false);
	static const TCHAR* GetPlatformFeaturesModuleName();

	FORCEINLINE static constexpr int32 GetMaxPathLength()
	{
		return IOS_MAX_PATH;
	}

	static bool AllowThreadHeartBeat();

	static EAppReturnType::Type MessageBoxExt( EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption );
	static void SetMemoryWarningHandler(void (* Handler)(const FGenericMemoryWarningContext& Context));
	static bool HasMemoryWarningHandler();
	static bool HasPlatformFeature(const TCHAR* FeatureName);
	static bool SetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, const FString& InValue);
	static bool GetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, FString& OutValue);
	static bool DeleteStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName);
	static bool DeleteStoredSection(const FString& InStoreId, const FString& InSectionName);
	static void GetValidTargetPlatforms(class TArray<class FString>& TargetPlatformNames);
	static const TCHAR* GamePersistentDownloadDir();
    static bool HasSeparateChannelForDebugOutput();

	static void RequestExit(bool Force, const TCHAR* CallSite = nullptr);
	static void RequestExitWithStatus(bool Force, uint8 ReturnCode, const TCHAR* CallSite = nullptr);

	UE_DEPRECATED(4.21, "Use GetDeviceVolume, it is now callable on all platforms.")
	static int GetAudioVolume();

	static bool AreHeadphonesPluggedIn();
	static int GetBatteryLevel();
	static bool IsRunningOnBattery();
	static float GetDeviceTemperatureLevel();
	static EDeviceThermalState GetDeviceThermalState();
	static EDeviceScreenOrientation GetDeviceOrientation();
	static void SetAllowedDeviceOrientation(EDeviceScreenOrientation NewAllowedDeviceOrientation);
	static int32 GetDeviceVolume();
	static void SetBrightness(float Brightness);
	static float GetBrightness();
	static bool SupportsBrightness() { return true; }
    static bool IsInLowPowerMode();

	// These two functions can be used to see if we're in designed for ipad mode explicltly
	// on vision os or mac. This check may not be valid in the future, but will do the trick for now
	static bool IsDesignedForIpadOnVisionOS();
	static bool IsDesignedForIpadOnMacOS();

	//////// Notifications
	static void RegisterForRemoteNotifications();
	static bool IsRegisteredForRemoteNotifications();
	static void UnregisterForRemoteNotifications();
	// Check if notifications are allowed if min iOS version is < 10
	UE_DEPRECATED(4.21, "IsAllowedRemoteNotifications is deprecated. Use FIOSLocalNotificationService::CheckAllowedNotifications instead.")
	static bool IsAllowedRemoteNotifications();

	enum EIOSAuthNotificationStatus
	{
		NotDetermined,
		Denied,
		Authorized,
		Provisional,
		Ephemeral,
		Unknown
	};
	static EIOSAuthNotificationStatus GetNotificationAuthorizationStatus();
    
	static FString GetPendingActivationProtocol();
	
    static bool IsEntitlementEnabled(const char *EntitlementToCheck);
	
	static class IPlatformChunkInstall* GetPlatformChunkInstall();

	static bool SupportsForceTouchInput();

	static void PrepareMobileHaptics(EMobileHapticsType Type);
	static void TriggerMobileHaptics();
	static void ReleaseMobileHaptics();

	static void ShareURL(const FString& URL, const FText& Description, int32 LocationHintX, int32 LocationHintY);

	static FString LoadTextFileFromPlatformPackage(const FString& RelativePath);
	static bool FileExistsInPlatformPackage(const FString& RelativePath);

	static void EnableVoiceChat(bool bEnable);
	static bool IsVoiceChatEnabled();
	static bool HasRecordPermission();

	/**
	 * Tri-state microphone (Apple "record") permission query, parallel to GetNotificationAuthorizationStatus.
	 * HasRecordPermission can't distinguish "user explicitly denied" from "user has never been asked", which
	 * callers need in order to decide between triggering the OS popup and opening app settings.
	 */
	enum class EIOSAuthMicrophonePermissionStatus : uint8
	{
		NotDetermined,
		Denied,
		Granted,
		Unknown
	};
	static EIOSAuthMicrophonePermissionStatus GetMicrophonePermissionStatus();

	//////// Platform specific
	static int GetDefaultStackSize();
	static void HandleLowMemoryWarning();
	static bool IsPackagedForDistribution();
	/**
	 * Implemented using UIDevice::identifierForVendor,
	 * so all the caveats that apply to that API call apply here.
	 */
	static FString GetDeviceId();
	static FString GetOSVersion();
	static FString GetUniqueAdvertisingId();

	// bInIncludeFreeable determines whether to use the "Important" iOS free space value or the actual free space (Capacity). 
	// The "Important" value is almost certainly what you want - but it's slow (upwards of 100ms).
	static bool GetDiskTotalAndFreeSpace(const FString& InPath, uint64& OutTotalNumberOfBytes, uint64& OutNumberOfFreeBytes, bool bInIncludeFreeable = true);
	
	static void RequestStoreReview();

	static bool IsUpdateAvailable();
	
	// GetIOSDeviceType is deprecated in 4.26 and is no longer updated. See below.
	enum EIOSDevice
	{
		// add new devices to the top, and add to IOSDeviceNames below!
		IOS_IPhone4,
		IOS_IPhone4S,
		IOS_IPhone5, // also the IPhone5C
		IOS_IPhone5S,
		IOS_IPodTouch5,
		IOS_IPodTouch6,
		IOS_IPad2,
		IOS_IPad3,
		IOS_IPad4,
		IOS_IPadMini,
		IOS_IPadMini2, // also the iPadMini3
		IOS_IPadMini4,
		IOS_IPadAir,
		IOS_IPadAir2,
		IOS_IPhone6,
		IOS_IPhone6Plus,
		IOS_IPhone6S,
		IOS_IPhone6SPlus,
        IOS_IPhone7,
        IOS_IPhone7Plus,
		IOS_IPhone8,
		IOS_IPhone8Plus,
		IOS_IPhoneX,
		IOS_IPadPro,
		IOS_AppleTV,
		IOS_AppleTV4K,
		IOS_IPhoneSE,
		IOS_IPadPro_129,
		IOS_IPadPro_97,
		IOS_IPadPro_105,
		IOS_IPadPro2_129,
		IOS_IPad5,
        IOS_IPhoneXS,
        IOS_IPhoneXSMax,
        IOS_IPhoneXR,
		IOS_IPhone11,
		IOS_IPhone11Pro,
		IOS_IPhone11ProMax,
		IOS_IPad6,
		IOS_IPadPro_11,
		IOS_IPadPro3_129,
        IOS_IPadAir3,
        IOS_IPadMini5,
		IOS_IPodTouch7,
		IOS_IPad7,
		IOS_IPhoneSE2,
		IOS_IPadPro2_11,
		IOS_IPadPro4_129,

		// We can use the entries below for any iOS devices released during the hotfix cycle
		// They should be moved to real device enum above these values in the next full release.
		IOS_NewDevice1,
		IOS_NewDevice2,
		IOS_NewDevice3,
		IOS_NewDevice4,
		IOS_NewDevice5,
		IOS_NewDevice6,
		IOS_NewDevice7,
		IOS_NewDevice8,

		IOS_RealityPro,
		
		IOS_Unknown,
	};

	UE_DEPRECATED(4.26, "Use GetDefaultDeviceProfileName() which uses the [IOSDeviceMappings] entries in BaseDeviceProfiles.ini and can be updated to support newly released devices.")
	static EIOSDevice GetIOSDeviceType();

	static const TCHAR* GetDefaultDeviceProfileName();

	static FString GetCPUVendor();
	static FString GetCPUBrand();
	static void GetOSVersions(FString& out_OSVersionLabel, FString& out_OSSubVersionLabel);
	static int32 IOSVersionCompare(uint8 Major, uint8 Minor, uint8 Revision);
	static FString GetProjectVersion();
	static FString GetBuildNumber();

    /**
     * @return true if the app is able to refresh in the background
     */
    static bool IsBackgroundAppRefreshAvailable();

    /**
     * Open the Settings app at the Notification page for this app.
     */
    static void OpenAppNotificationSettings();
	
    /**
     * Open the app's custom settings in the Settings App.
     */
    static void OpenAppCustomSettings();

	static void SetGracefulTerminationHandler();
	static void SetCrashHandler(void(*CrashHandler)(const FGenericCrashContext& Context));

	static bool SupportsDeviceCheckToken()
	{
		return true;
	}

	static bool RequestDeviceCheckToken(TFunction<void(const TArray<uint8>&)> QuerySucceededFunc, TFunction<void(const FString&, const FString&)> QueryFailedFunc);
	
   
	FORCEINLINE static void ChooseHDRDeviceAndColorGamut(uint32 DeviceId, uint32 DisplayNitLevel, EDisplayOutputFormat& OutputDevice, EDisplayColorGamut& ColorGamut)
	{
		// Linear output to Apple's specific format.
		OutputDevice = EDisplayOutputFormat::HDR_LinearEXR;
		ColorGamut = EDisplayColorGamut::sRGB_D65;
	}

	static float GetVirtualKeyboardInputHeight();

	static int32 GetMaxRefreshRate();

    // added these for now because Crashlytics doesn't properly break up different callstacks all ending in UE_LOG(LogXXX, Fatal, ...)
    static FORCENOINLINE CA_NO_RETURN void GPUAssert();
    static FORCENOINLINE CA_NO_RETURN void MetalAssert();

	static bool CPUHasHwCrcSupport();
	static bool CPUHasHwAesSupport();

	static FString GetDiscardableCacheDir();

#if !UE_BUILD_SHIPPING
	static bool IsConsoleOpen();
#endif

	static const class TMap<FString, FString>& GetConfigRuleVars();
};

typedef FIOSPlatformMisc FPlatformMisc;
