// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Apple/AppleRecursiveMutex.h"
#include "GenericPlatform/GenericPlatformMutex.h"
#include "HAL/PThreadsSharedMutex.h"

namespace UE
{

using FPlatformRecursiveMutex = FAppleRecursiveMutex;
using FPlatformSharedMutex = FPThreadsSharedMutex;
using FPlatformSystemWideMutex = FPlatformSystemWideMutexNotImplemented;

} // UE
