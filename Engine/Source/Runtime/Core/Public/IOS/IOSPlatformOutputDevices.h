// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	IOSPlatformOutputDevices.h: iOS platform OutputDevices functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformOutputDevices.h"
#include "Misc/OutputDeviceFile.h"

struct CORE_API FIOSPlatformOutputDevices : public FGenericPlatformOutputDevices
{
    static FOutputDevice*		GetLog();
};

typedef FIOSPlatformOutputDevices FPlatformOutputDevices;


class FIOSOutputDeviceFile : public FOutputDeviceFile
{
public:
	FIOSOutputDeviceFile(const TCHAR* InFilename = nullptr, bool bDisableBackup = false, bool bAppendIfExists = false);

	virtual void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category, const double Time) override;
};
