// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	IOSPlatformPLCrashReporterIncludes.h: Wrapper for the third party PLCrashReporter includes for IOS
==============================================================================================*/

#pragma once

THIRD_PARTY_INCLUDES_START
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
#include "PLCrashReporter.h"
#include "PLCrashReport.h"
#include "PLCrashReportTextFormatter.h"
#endif
THIRD_PARTY_INCLUDES_END
