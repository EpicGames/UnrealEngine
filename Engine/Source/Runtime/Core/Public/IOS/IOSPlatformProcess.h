// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	IOSPlatformProcess.h: iOS platform Process functions
==============================================================================================*/

#pragma once
#include "Apple/ApplePlatformProcess.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#import "IOSAsyncTask.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

/** Dummy process handle for platforms that use generic implementation. */
struct FProcHandle : public TProcHandle<void*, nullptr>
{
public:
	/** Default constructor. */
	UE_FORCEINLINE_HINT constexpr FProcHandle()
		: TProcHandle()
	{}

	/** Initialization constructor. */
	UE_FORCEINLINE_HINT explicit FProcHandle( HandleType Other )
		: TProcHandle( Other )
	{}
};

/**
* iOS implementation of the Process OS functions
**/
struct CORE_API FIOSPlatformProcess : public FApplePlatformProcess
{
	static const TCHAR* ComputerName();
	static const TCHAR* BaseDir();
	static FRunnableThread* CreateRunnableThread();
	static void LaunchURL( const TCHAR* URL, const TCHAR* Parms, FString* Error );
	static void LaunchSyncURL( const TCHAR* URL, const TCHAR* Parms, float Timeout, FString* Error );
	static bool CanLaunchURL(const TCHAR* URL);
	static FString GetGameBundleId();
	static void SetupGameThread();
	static void SetupRenderThread();
	static void SetThreadAffinityMask(uint64 AffinityMask);
	static const TCHAR* ExecutablePath();
	static const TCHAR* ExecutableName(bool bRemoveExtension = true);
    static FString GenerateApplicationPath( const FString& AppName, EBuildConfiguration BuildConfiguration);

private:
	static void SetupThread(int Priority);
};

typedef FIOSPlatformProcess FPlatformProcess;
