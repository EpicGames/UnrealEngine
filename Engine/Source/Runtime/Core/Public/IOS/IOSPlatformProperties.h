// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	IOSPlatformProperties.h - Basic static properties of a platform 
	These are shared between:
		the runtime platform - via FPlatformProperties
		the target platforms - via ITargetPlatform
==================================================================================*/

#pragma once

#include "GenericPlatform/GenericPlatformProperties.h"


/**
 * Implements iOS platform properties.
 */
struct FIOSPlatformProperties
	: public FGenericPlatformProperties
{
	static constexpr UE_FORCEINLINE_HINT bool HasEditorOnlyData()
	{
		return false;
	}

	static constexpr UE_FORCEINLINE_HINT const char* PlatformName()
	{
		return "IOS";
	}

	static constexpr UE_FORCEINLINE_HINT const char* IniPlatformName()
	{
		return "IOS";
	}

	static constexpr UE_FORCEINLINE_HINT const TCHAR* GetRuntimeSettingsClassName()
	{
		return TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings");
	}

	static constexpr UE_FORCEINLINE_HINT bool IsGameOnly()
	{
		return true;
	}
	
	static constexpr UE_FORCEINLINE_HINT bool RequiresCookedData()
	{
		return true;
	}
    
	static UE_FORCEINLINE_HINT bool SupportsBuildTarget(EBuildTargetType TargetType)
	{
		return (TargetType == EBuildTargetType::Game);
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsLowQualityLightmaps()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsHighQualityLightmaps()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsTextureStreaming()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMemoryMappedFiles()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMemoryMappedAudio()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMemoryMappedAnimation()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT int64 GetMemoryMappingAlignment()
	{
		return 16384;
	}

	static constexpr UE_FORCEINLINE_HINT int GetMaxSupportedVirtualMemoryAlignment()
	{
		return 16384;
	}

	static constexpr UE_FORCEINLINE_HINT bool HasFixedResolution()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool AllowsFramerateSmoothing()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsAudioStreaming()
	{
		return true;
	}
	
	static constexpr UE_FORCEINLINE_HINT bool SupportsMeshLODStreaming()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsQuit()
	{
		return true;
	}
	
	static constexpr UE_FORCEINLINE_HINT bool SupportsRayTracing()
	{
		return true;
	}
};

struct FTVOSPlatformProperties : public FIOSPlatformProperties
{
	// @todo breaking change here!
	static constexpr UE_FORCEINLINE_HINT const char* PlatformName()
	{
		return "TVOS";
	}

	static constexpr UE_FORCEINLINE_HINT const char* IniPlatformName()
	{
		return "TVOS";
	}
};

struct FVisionOSPlatformProperties : public FIOSPlatformProperties
{
	static constexpr UE_FORCEINLINE_HINT const char* PlatformName()
	{
		return "IOS";
	}

	static constexpr UE_FORCEINLINE_HINT const char* IniPlatformName()
	{
		return "VisionOS";
	}
};

#ifdef PROPERTY_HEADER_SHOULD_DEFINE_TYPE

#if PLATFORM_VISIONOS
typedef FVisionOSPlatformProperties FPlatformProperties;
#else
typedef FIOSPlatformProperties FPlatformProperties;
#endif

#endif
