// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/IBreakIterator.h"

struct FBreakIterator
{
	/**
	 * Create a new instance of a break iterator designed to locate character boundary points within a string
	 */
	static CORE_API TSharedRef<IBreakIterator> CreateCharacterBoundaryIterator();

	/**
	 * Create a new instance of a break iterator designed to locate word boundary points within a string
	 */
	static CORE_API TSharedRef<IBreakIterator> CreateWordBreakIterator();

	/**
	 * Create a new instance of a break iterator designed to locate appropriate soft-wrapping points within a string
	 * Note that these aren't newlines as such, but rather places where the line could be soft-wrapped when displaying the text
	 */
	static CORE_API TSharedRef<IBreakIterator> CreateLineBreakIterator();

	/**
	 * Create a new instance of a break iterator which is designed to locate appropriate soft-wrapping points within a string by searching for CamelCase boundaries
	 * This may produce odd results when used with languages which don't have a concept of cases for characters, however should be fine for parsing variable or asset names
	 */
	static CORE_API TSharedRef<IBreakIterator> CreateCamelCaseBreakIterator();

	/**
	 * Get a shared instance of a character boundary break iterator
	 */
	static CORE_API TSharedRef<IBreakIterator> GetCharacterBoundaryIterator();

	/**
	 * Get a shared instance of a work break iterator
	 */
	static CORE_API TSharedRef<IBreakIterator> GetWordBreakIterator();

	/**
	 * Get a new instance of a line break iterator 
	 */
	static CORE_API TSharedRef<IBreakIterator> GetLineBreakIterator();
};
