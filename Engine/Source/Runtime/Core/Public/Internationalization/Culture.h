// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/UniquePtr.h"
#include "Internationalization/CulturePointer.h"

struct FDecimalNumberFormattingRules;
enum class ETextPluralForm : uint8;
enum class ETextPluralType : uint8;

#if UE_ENABLE_ICU
class FICUCultureImplementation;
typedef FICUCultureImplementation FCultureImplementation;
#else
class FLegacyCultureImplementation;
typedef FLegacyCultureImplementation FCultureImplementation;
#endif

enum class ECultureIdentifierType : uint8
{
	/** UE culture name (eg, "en-US") */
	UE,
	/** Verse culture identifier (eg, "en_US") */
	Verse,
};

class FCulture
{
#if UE_ENABLE_ICU
	friend class FText;
	friend class FTextChronoFormatter;
	friend class FTextComparison;
	friend class FICUBreakIteratorManager;
#endif

public:
	CORE_API ~FCulture();
	
	static CORE_API FCultureRef Create(TUniquePtr<FCultureImplementation>&& InImplementation);

	CORE_API const FString& GetDisplayName() const;

	CORE_API const FString& GetEnglishName() const;

	CORE_API int GetKeyboardLayoutId() const;

	CORE_API int GetLCID() const;

	CORE_API TArray<FString> GetPrioritizedParentCultureNames() const;

	static CORE_API TArray<FString> GetPrioritizedParentCultureNames(const FString& LanguageCode, const FString& ScriptCode, const FString& RegionCode);

	static CORE_API FString CreateCultureName(const FString& LanguageCode, const FString& ScriptCode, const FString& RegionCode);

	static CORE_API FString GetCanonicalName(const FString& Name);

	/**
	 * Given a well-formed UE culture name (eg, "en-US"), convert it to its equivalent Verse identifier (eg, "en_US").
	 */
	static CORE_API FString CultureNameToVerseIdentifier(const FString& Name);

	/**
	 * Given a well-formed Verse culture identifier (eg, "en_US"), convert it to its equivalent UE culture name (eg, "en-US").
	 * @note GetCanonicalName will also perform this conversion, with additional validation.
	 */
	static CORE_API FString CultureNameFromVerseIdentifier(const FString& VerseIdentifer);

	/**
	 * Get the preferred on-disk identifier type (eg, "en-US" vs "en_US") to use when creating new localized content.
	 * @note You should still assume both kinds of identifier may be used within a project.
	 */
	static CORE_API ECultureIdentifierType GetPreferredOnDiskIdentifierType();

	/**
	 * Get the preferred on-disk identifier of this culture (eg, "en-US" vs "en_US") to use when creating new localized content.
	 * @note You should still assume both kinds of identifier may be used within a project.
	 */
	CORE_API const FString& GetPreferredOnDiskIdentifier() const;

	/**
	 * Get the name of this culture, eg, "en-US"
	 */
	CORE_API const FString& GetName() const;
	
	/**
	 * Get the name of this culture as a Verse identifier (for a Verse path), eg, "en_US"
	 */
	CORE_API const FString& GetVerseIdentifier() const;

	CORE_API const FString& GetNativeName() const;

	CORE_API const FString& GetUnrealLegacyThreeLetterISOLanguageName() const;

	CORE_API const FString& GetThreeLetterISOLanguageName() const;

	CORE_API const FString& GetTwoLetterISOLanguageName() const;

	CORE_API const FString& GetNativeLanguage() const;

	CORE_API const FString& GetRegion() const;

	CORE_API const FString& GetNativeRegion() const;

	CORE_API const FString& GetScript() const;

	CORE_API const FString& GetVariant() const;

	CORE_API bool IsRightToLeft() const;

	CORE_API const FDecimalNumberFormattingRules& GetDecimalNumberFormattingRules() const;

	CORE_API const FDecimalNumberFormattingRules& GetPercentFormattingRules() const;

	CORE_API const FDecimalNumberFormattingRules& GetCurrencyFormattingRules(const FString& InCurrencyCode) const;

	/**
	 * Get the correct plural form to use for the given number
	 * @param PluralType The type of plural form to get (cardinal or ordinal)
	 */
	CORE_API ETextPluralForm GetPluralForm(float Val,	const ETextPluralType PluralType) const;
	CORE_API ETextPluralForm GetPluralForm(double Val,	const ETextPluralType PluralType) const;
	CORE_API ETextPluralForm GetPluralForm(int8 Val,		const ETextPluralType PluralType) const;
	CORE_API ETextPluralForm GetPluralForm(int16 Val,	const ETextPluralType PluralType) const;
	CORE_API ETextPluralForm GetPluralForm(int32 Val,	const ETextPluralType PluralType) const;
	CORE_API ETextPluralForm GetPluralForm(int64 Val,	const ETextPluralType PluralType) const;
	CORE_API ETextPluralForm GetPluralForm(uint8 Val,	const ETextPluralType PluralType) const;
	CORE_API ETextPluralForm GetPluralForm(uint16 Val,	const ETextPluralType PluralType) const;
	CORE_API ETextPluralForm GetPluralForm(uint32 Val,	const ETextPluralType PluralType) const;
	CORE_API ETextPluralForm GetPluralForm(uint64 Val,	const ETextPluralType PluralType) const;
	CORE_API ETextPluralForm GetPluralForm(long Val,		const ETextPluralType PluralType) const;

	/**
	 * Get the plural forms supported by this culture
	 * @param PluralType The type of plural form to get (cardinal or ordinal)
	 */
	CORE_API const TArray<ETextPluralForm>& GetValidPluralForms(const ETextPluralType PluralType) const;
	
	CORE_API void RefreshCultureDisplayNames(const TArray<FString>& InPrioritizedDisplayCultureNames, const bool bFullRefresh = true);

private:
	CORE_API explicit FCulture(TUniquePtr<FCultureImplementation>&& InImplementation);

	TUniquePtr<FCultureImplementation> Implementation;

	FString CachedDisplayName;
	FString CachedEnglishName;
	FString CachedName;
	FString CachedVerseIdentifier;
	FString CachedNativeName;
	FString CachedUnrealLegacyThreeLetterISOLanguageName;
	FString CachedThreeLetterISOLanguageName;
	FString CachedTwoLetterISOLanguageName;
	FString CachedNativeLanguage;
	FString CachedRegion;
	FString CachedNativeRegion;
	FString CachedScript;
	FString CachedVariant;
	bool CachedIsRightToLeft;
};
