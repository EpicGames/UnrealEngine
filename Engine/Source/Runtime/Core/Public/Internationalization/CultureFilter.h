// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Set.h"
#include "Containers/UnrealString.h"

enum class EBuildConfiguration : uint8;
enum class ELocalizationLoadFlags : uint8;

/** Cache of enabled/disabled cultures loaded from config data */
class FCultureFilter
{
public:
	/** Construct a culture filter based on the current build config and target, optionally filtering any loaded cultures against the set of available cultures */
	CORE_API explicit FCultureFilter(const TSet<FString>* AvailableCultures = nullptr);

	/** Construct a culture filter based on the given build config and target, optionally filtering any loaded cultures against the set of available cultures */
	CORE_API FCultureFilter(const EBuildConfiguration BuildConfig, const ELocalizationLoadFlags TargetFlags, const TSet<FString>* AvailableCultures = nullptr);

	/** Does the given culture pass this filter? */
	CORE_API bool IsCultureAllowed(const FString& Culture) const;

private:
	void Init(const EBuildConfiguration BuildConfig, const ELocalizationLoadFlags TargetFlags, const TSet<FString>* AvailableCultures);

	TSet<FString> EnabledCultures;
	TSet<FString> DisabledCultures;
};
