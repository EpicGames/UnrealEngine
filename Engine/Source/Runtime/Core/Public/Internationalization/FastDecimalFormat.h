// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/Integral.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Math/NumericLimits.h"
#include "Misc/CString.h"

#include <type_traits>

/** Rules used to format or parse a decimal number */
struct FDecimalNumberFormattingRules
{
	FDecimalNumberFormattingRules()
		: GroupingSeparatorCharacter(TEXT('\0'))
		, DecimalSeparatorCharacter(TEXT('\0'))
		, PrimaryGroupingSize(0)
		, SecondaryGroupingSize(0)
		, MinimumGroupingDigits(1)
	{
		DigitCharacters[0] = TEXT('0');
		DigitCharacters[1] = TEXT('1');
		DigitCharacters[2] = TEXT('2');
		DigitCharacters[3] = TEXT('3');
		DigitCharacters[4] = TEXT('4');
		DigitCharacters[5] = TEXT('5');
		DigitCharacters[6] = TEXT('6');
		DigitCharacters[7] = TEXT('7');
		DigitCharacters[8] = TEXT('8');
		DigitCharacters[9] = TEXT('9');
	}

	/** Number formatting rules, typically extracted from the ICU decimal formatter for a given culture */
	FString NaNString;
	FString NegativePrefixString;
	FString NegativeSuffixString;
	FString PositivePrefixString;
	FString PositiveSuffixString;
	FString PlusString;
	FString MinusString;
	TCHAR GroupingSeparatorCharacter;
	TCHAR DecimalSeparatorCharacter;
	uint8 PrimaryGroupingSize;
	uint8 SecondaryGroupingSize;
	uint8 MinimumGroupingDigits;
	TCHAR DigitCharacters[10];

	/** Default number formatting options for a given culture */
	FNumberFormattingOptions CultureDefaultFormattingOptions;
};

/**
 * Provides efficient and culture aware number formatting and parsing.
 * You would call FastDecimalFormat::NumberToString to convert a number to the correct decimal representation based on the given formatting rules and options.
 * You would call FastDecimalFormat::StringToNumber to convert a string containing a culture correct decimal representation of a number into an actual number.
 * The primary consumer of this is FText, however you can use it for other things. GetCultureAgnosticFormattingRules can provide formatting rules for cases where you don't care about culture.
 * @note If you use the version of FastDecimalFormat::NumberToString that takes an output string, the formatted number will be appended to the existing contents of the string.
 */
namespace FastDecimalFormat
{

namespace Internal
{

CORE_API void IntegralToString(const bool bIsNegative, const uint64 InVal, const FDecimalNumberFormattingRules& InFormattingRules, FNumberFormattingOptions InFormattingOptions, FString& OutString);
CORE_API void FractionalToString(const double InVal, const FDecimalNumberFormattingRules& InFormattingRules, FNumberFormattingOptions InFormattingOptions, FString& OutString);

struct FDecimalNumberIntegralLimits
{
	FDecimalNumberIntegralLimits(int64 InLowest, uint64 InMax, bool bInIsSigned)
		: NumericLimitLowest(InLowest), NumericLimitMax(InMax), bIsNumericSigned(bInIsSigned)
	{ }

	int64 NumericLimitLowest;
	uint64 NumericLimitMax;
	bool bIsNumericSigned;

	template <UE::CIntegral IntegralType>
	static FDecimalNumberIntegralLimits FromNumericLimits()
	{
		return FDecimalNumberIntegralLimits(TNumericLimits<IntegralType>::Lowest(), TNumericLimits<IntegralType>::Max(), std::is_signed_v<IntegralType>);
	}
};

struct FDecimalNumberFractionalLimits
{
	FDecimalNumberFractionalLimits(double InLowest, double InMax)
		: NumericLimitLowest(InLowest), NumericLimitMax(InMax)
	{ }

	double NumericLimitLowest;
	double NumericLimitMax;

	template <
		typename FloatingType
		UE_REQUIRES(std::is_floating_point_v<FloatingType>)
	>
	static FDecimalNumberFractionalLimits FromNumericLimits()
	{
		return FDecimalNumberFractionalLimits(TNumericLimits<FloatingType>::Lowest(), TNumericLimits<FloatingType>::Max());
	}
};

CORE_API bool StringToIntegral(const TCHAR* InStr, const int32 InStrLen, const FDecimalNumberFormattingRules& InFormattingRules, const FNumberParsingOptions& InParsingOptions, const FDecimalNumberIntegralLimits& Limits, bool& OutIsNegative, uint64& OutVal, int32* OutParsedLen);
CORE_API bool StringToFractional(const TCHAR* InStr, const int32 InStrLen, const FDecimalNumberFormattingRules& InFormattingRules, const FNumberParsingOptions& InParsingOptions, const FDecimalNumberFractionalLimits& Limits, double& OutVal, int32* OutParsedLen);

} // namespace Internal

template<typename T>
inline void NumberToString(const T InVal, const FDecimalNumberFormattingRules& InFormattingRules, const FNumberFormattingOptions& InFormattingOptions, FString& OutString)
{
	if constexpr (std::is_same_v<T, int8> || std::is_same_v<T, int16> || std::is_same_v<T, int32> || std::is_same_v<T, int64>)
	{
		#ifdef _MSC_VER
		#pragma warning (push)
		#pragma warning (disable : 4146) // unary minus operator applied to unsigned type, result still unsigned
		#endif
		const bool bIsNegative = InVal < 0;
		Internal::IntegralToString(bIsNegative, (bIsNegative) ? -static_cast<uint64>(InVal) : static_cast<uint64>(InVal), InFormattingRules, InFormattingOptions, OutString);
		#ifdef _MSC_VER
		#pragma warning (pop)
		#endif
	}
	else if constexpr (std::is_same_v<T, uint8> || std::is_same_v<T, uint16> || std::is_same_v<T, uint32> || std::is_same_v<T, uint64>)
	{
		Internal::IntegralToString(false, static_cast<uint64>(InVal), InFormattingRules, InFormattingOptions, OutString);
	}
	else if constexpr (std::is_same_v<T, float> || std::is_same_v<T, double>)
	{
		Internal::FractionalToString(static_cast<double>(InVal), InFormattingRules, InFormattingOptions, OutString);
	}
	else
	{
		static_assert(sizeof(T) == 0, "Not supported");
	}
}

template<typename T>
inline FString NumberToString(const T InVal, const FDecimalNumberFormattingRules& InFormattingRules, const FNumberFormattingOptions& InFormattingOptions)						\
{
	FString Result;																																												\
	NumberToString(InVal, InFormattingRules, InFormattingOptions, Result);																														\
	return Result;																																												\
}

#define FAST_DECIMAL_PARSE_INTEGER_IMPL(NUMBER_TYPE)																																														\
	inline bool StringToNumber(const TCHAR* InStr, const int32 InStrLen, const FDecimalNumberFormattingRules& InFormattingRules, const FNumberParsingOptions& InParsingOptions, NUMBER_TYPE& OutVal, int32* OutParsedLen = nullptr)	\
	{																																																										\
		bool bIsNegative = false;																																																			\
		uint64 Val = 0;																																																						\
		const bool bResult = Internal::StringToIntegral(InStr, InStrLen, InFormattingRules, InParsingOptions, Internal::FDecimalNumberIntegralLimits::FromNumericLimits<NUMBER_TYPE>(), bIsNegative, Val, OutParsedLen);								\
		OutVal = static_cast<NUMBER_TYPE>(Val);																																																\
		OutVal *= (bIsNegative ? -1 : 1);																																																	\
		return bResult;																																																						\
	}																																																										\
	UE_FORCEINLINE_HINT bool StringToNumber(const TCHAR* InStr, const FDecimalNumberFormattingRules& InFormattingRules, const FNumberParsingOptions& InParsingOptions, NUMBER_TYPE& OutVal, int32* OutParsedLen = nullptr)							\
	{																																																										\
		return StringToNumber(InStr, FCString::Strlen(InStr), InFormattingRules, InParsingOptions, OutVal, OutParsedLen);																													\
	}

#define FAST_DECIMAL_PARSE_FRACTIONAL_IMPL(NUMBER_TYPE)																																														\
	inline bool StringToNumber(const TCHAR* InStr, const int32 InStrLen, const FDecimalNumberFormattingRules& InFormattingRules, const FNumberParsingOptions& InParsingOptions, NUMBER_TYPE& OutVal, int32* OutParsedLen = nullptr)	\
	{																																																										\
		double Val = 0.0;																																																					\
		const bool bResult = Internal::StringToFractional(InStr, InStrLen, InFormattingRules, InParsingOptions, Internal::FDecimalNumberFractionalLimits::FromNumericLimits<NUMBER_TYPE>(), Val, OutParsedLen);																											\
		OutVal = static_cast<NUMBER_TYPE>(Val);																																																\
		return bResult;																																																						\
	}																																																										\
	UE_FORCEINLINE_HINT bool StringToNumber(const TCHAR* InStr, const FDecimalNumberFormattingRules& InFormattingRules, const FNumberParsingOptions& InParsingOptions, NUMBER_TYPE& OutVal, int32* OutParsedLen = nullptr)							\
	{																																																										\
		return StringToNumber(InStr, FCString::Strlen(InStr), InFormattingRules, InParsingOptions, OutVal, OutParsedLen);																													\
	}

FAST_DECIMAL_PARSE_INTEGER_IMPL(int8)
FAST_DECIMAL_PARSE_INTEGER_IMPL(int16)
FAST_DECIMAL_PARSE_INTEGER_IMPL(int32)
FAST_DECIMAL_PARSE_INTEGER_IMPL(int64)

FAST_DECIMAL_PARSE_INTEGER_IMPL(uint8)
FAST_DECIMAL_PARSE_INTEGER_IMPL(uint16)
FAST_DECIMAL_PARSE_INTEGER_IMPL(uint32)
FAST_DECIMAL_PARSE_INTEGER_IMPL(uint64)

FAST_DECIMAL_PARSE_FRACTIONAL_IMPL(float)
FAST_DECIMAL_PARSE_FRACTIONAL_IMPL(double)

#undef FAST_DECIMAL_PARSE_INTEGER_IMPL
#undef FAST_DECIMAL_PARSE_FRACTIONAL_IMPL

/**
 * Get the formatting rules to use when you don't care about culture.
 */
CORE_API const FDecimalNumberFormattingRules& GetCultureAgnosticFormattingRules();

/**
 * Return the value of 10^exp for the given exponent value.
 * @note The maximum exponent supported is 10^18.
 */
CORE_API uint64 Pow10(const int32 InExponent);

} // namespace FastDecimalFormat
