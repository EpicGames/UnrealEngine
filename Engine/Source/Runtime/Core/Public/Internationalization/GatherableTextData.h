// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
 
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Internationalization/InternationalizationMetadata.h"
#include "Serialization/StructuredArchive.h"

class FArchive;

struct FTextSourceSiteContext
{
	FTextSourceSiteContext()
		: IsEditorOnly(false)
		, IsOptional(false)
	{
	}

	FString KeyName;
	FString SiteDescription;
	bool IsEditorOnly;
	bool IsOptional;
#if WITH_EDITORONLY_DATA
	FString DevNotes;
#endif
	FLocMetadataObject InfoMetaData;
	FLocMetadataObject KeyMetaData;

	friend FArchive& operator<<(FArchive& Archive, FTextSourceSiteContext& This) { This.StreamArchive(Archive); return Archive; }
	friend void operator<<(FStructuredArchive::FSlot Slot, FTextSourceSiteContext& This) { This.StreamStructuredArchive(Slot); }

	CORE_API void StreamArchive(FArchive& Archive);
	CORE_API void StreamStructuredArchive(FStructuredArchive::FSlot Slot);
};

struct FTextSourceData
{
	FString SourceString;
	FLocMetadataObject SourceStringMetaData;

	friend FArchive& operator<<(FArchive& Archive, FTextSourceData& This) { This.StreamArchive(Archive); return Archive; }
	friend void operator<<(FStructuredArchive::FSlot Slot, FTextSourceData& This) { This.StreamStructuredArchive(Slot); }

	CORE_API void StreamArchive(FArchive& Archive);
	CORE_API void StreamStructuredArchive(FStructuredArchive::FSlot Slot);
};

struct FGatherableTextData
{
	FString NamespaceName;
	FTextSourceData SourceData;

	TArray<FTextSourceSiteContext> SourceSiteContexts;

	friend FArchive& operator<<(FArchive& Archive, FGatherableTextData& This) { This.StreamArchive(Archive); return Archive; }
	friend void operator<<(FStructuredArchive::FSlot Slot, FGatherableTextData& This) { This.StreamStructuredArchive(Slot); }

	CORE_API void StreamArchive(FArchive& Archive);
	CORE_API void StreamStructuredArchive(FStructuredArchive::FSlot Slot);
};
