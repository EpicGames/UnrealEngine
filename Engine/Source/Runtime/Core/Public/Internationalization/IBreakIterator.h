// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/StringView.h"
#include "Internationalization/Text.h"

class IBreakIterator
{
public:
	virtual ~IBreakIterator() = default;

	virtual void SetString(const FText& InText)
	{
		SetString(CopyTemp(InText.ToString()));
	}
	virtual void SetString(const FString& InString)
	{
		SetString(CopyTemp(InString));
	}
	virtual void SetString(const TCHAR* const InString, const int32 InStringLength)
	{
		SetString(FString::ConstructFromPtrSize(InString, (!InString || *InString == TEXT('\0')) ? 0 : InStringLength));
	}
	virtual void SetString(FString&& InString) = 0;
	
	virtual void SetStringRef(const FText& InText)
	{
		SetStringRef(FStringView(InText.ToString()));
	}
	virtual void SetStringRef(const FString& InString)
	{
		SetStringRef(FStringView(InString));
	}
	virtual void SetStringRef(const TCHAR* const InString, const int32 InStringLength)
	{
		SetStringRef(FStringView(InString, InStringLength));
	}
	virtual void SetStringRef(const FString* InString)
	{
		check(InString);
		SetStringRef(FStringView(*InString));
	}
	virtual void SetStringRef(FStringView InString) = 0;

	virtual void ClearString()
	{
		SetStringRef(FStringView());
	}

	virtual int32 GetCurrentPosition() const = 0;

	virtual int32 ResetToBeginning() = 0;
	virtual int32 ResetToEnd() = 0;

	virtual int32 MoveToPrevious() = 0;
	virtual int32 MoveToNext() = 0;
	virtual int32 MoveToCandidateBefore(const int32 InIndex) = 0;
	virtual int32 MoveToCandidateAfter(const int32 InIndex) = 0;
};
