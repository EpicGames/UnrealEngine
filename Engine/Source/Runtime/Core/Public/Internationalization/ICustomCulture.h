// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Internationalization/CulturePointer.h"

/**
 * Interface for a custom culture.
 * @note See FCulture for the API description.
 */
class ICustomCulture
{
public:
	virtual ~ICustomCulture() = default;

	virtual FCultureRef GetBaseCulture() const = 0;
	virtual FString GetDisplayName() const = 0;
	virtual FString GetEnglishName() const = 0;
	virtual FString GetName() const = 0;
	virtual FString GetNativeName() const = 0;
	virtual FString GetUnrealLegacyThreeLetterISOLanguageName() const = 0;
	virtual FString GetThreeLetterISOLanguageName() const = 0;
	virtual FString GetTwoLetterISOLanguageName() const = 0;
	virtual FString GetNativeLanguage() const = 0;
	virtual FString GetNativeRegion() const = 0;
	virtual FString GetRegion() const = 0;
	virtual FString GetScript() const = 0;
	virtual FString GetVariant() const = 0;
	virtual bool IsRightToLeft() const = 0;
};
