// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "Containers/ArrayView.h"
#include "Internationalization/TextKey.h"
#include "Internationalization/LocalizedTextSourceTypes.h"
#include "CoreGlobals.h"

class FTextLocalizationResource;

/**
 * Interface for a localized text source.
 * These can be registered with the text localization manager, and provide an extensible way to inject localized text to UE4.
 */
class ILocalizedTextSource
{
public:
	virtual ~ILocalizedTextSource() = default;

	/**
	 * Get the priority of this source when building the complete list of translations to apply (higher numbers have a higher priority).
	 */
	virtual int32 GetPriority() const
	{
		return ELocalizedTextSourcePriority::Normal;
	}

	/**
	 * Given a localization category, get the native culture for the category (if known).
	 * @return True if the native culture was populated, or false if the native culture is unknown.
	 */
	virtual bool GetNativeCultureName(const ELocalizedTextSourceCategory InCategory, FString& OutNativeCultureName) = 0;

	/**
	 * Populate a list of culture names that this localized text source has resource data for (ELocalizationLoadFlags controls which resources should be checked).
	 */
	virtual void GetLocalizedCultureNames(const ELocalizationLoadFlags InLoadFlags, TSet<FString>& OutLocalizedCultureNames) = 0;

	/**
	 * Load the localized resources from this localized text source for the given cultures into the given maps (ELocalizationLoadFlags controls which resources should be loaded).
	 */
	virtual void LoadLocalizedResources(const ELocalizationLoadFlags InLoadFlags, TArrayView<const FString> InPrioritizedCultures, FTextLocalizationResource& InOutNativeResource, FTextLocalizationResource& InOutLocalizedResource) = 0;

	/**
	 * Query a localized resource from this localized text source for the given cultures and ID into the given maps (ELocalizationLoadFlags controls which resources should be queried).
	 */
	virtual EQueryLocalizedResourceResult QueryLocalizedResource(const ELocalizationLoadFlags InLoadFlags, TArrayView<const FString> InPrioritizedCultures, const FTextId InTextId, FTextLocalizationResource& InOutNativeResource, FTextLocalizationResource& InOutLocalizedResource)
	{
		return EQueryLocalizedResourceResult::NotImplemented;
	}

	/** Should we load native data based on the given load flags and environment? */
	static UE_FORCEINLINE_HINT bool ShouldLoadNative(const ELocalizationLoadFlags InLoadFlags)
	{
		return EnumHasAnyFlags(InLoadFlags, ELocalizationLoadFlags::Native);
	}

	/** Should we load editor data based on the given load flags and environment? */
	static UE_FORCEINLINE_HINT bool ShouldLoadEditor(const ELocalizationLoadFlags InLoadFlags)
	{
		return EnumHasAnyFlags(InLoadFlags, ELocalizationLoadFlags::Editor);
	}

	/** Should we load game data based on the given load flags and environment? */
	static UE_FORCEINLINE_HINT bool ShouldLoadGame(const ELocalizationLoadFlags InLoadFlags)
	{
		return EnumHasAnyFlags(InLoadFlags, ELocalizationLoadFlags::Game | ELocalizationLoadFlags::ForceLocalizedGame);
	}

	/** Should we load engine data based on the given load flags and environment? */
	static UE_FORCEINLINE_HINT bool ShouldLoadEngine(const ELocalizationLoadFlags InLoadFlags)
	{
		return EnumHasAnyFlags(InLoadFlags, ELocalizationLoadFlags::Engine);
	}

	/** Should we load additional (eg, plugin) data based on the given load flags and environment? */
	static UE_FORCEINLINE_HINT bool ShouldLoadAdditional(const ELocalizationLoadFlags InLoadFlags)
	{
		return EnumHasAnyFlags(InLoadFlags, ELocalizationLoadFlags::Additional);
	}

	/** Should we load native game data based on the given load flags and environment? */
	static UE_FORCEINLINE_HINT bool ShouldLoadNativeGameData(const ELocalizationLoadFlags InLoadFlags)
	{
		// The editor loads native game data by default to prevent authoring issues
		// It will load localized data only if the request is forced (eg, when entering game localization preview mode)
		return GIsEditor && !EnumHasAnyFlags(InLoadFlags, ELocalizationLoadFlags::ForceLocalizedGame);
	}
};
