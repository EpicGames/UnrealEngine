// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Internationalization/TextLocalizationManager.h"
#include "Templates/RefCounting.h"

class FTextHistory;

/** 
 * Interface to the internal data for an FText.
 */
class ITextData : public IRefCountedObject
{
public:
	virtual ~ITextData() = default;

	/**
	 * Get the source string for this text.
	 */
	virtual const FString& GetSourceString() const = 0;

	/**
	 * Get the string to use for display purposes.
	 * This may be a shared display string from the localization manager, or may been generated at runtime.
	 */
	virtual const FString& GetDisplayString() const = 0;

	/**
	 * Get the shared display string (if any).
	 */
	virtual FTextConstDisplayStringPtr GetLocalizedString() const = 0;

	/**
	 * Get the global history revision associated with this text instance.
	 */
	virtual uint16 GetGlobalHistoryRevision() const = 0;

	/**
	 * Get the local history revision associated with this text instance.
	 */
	virtual uint16 GetLocalHistoryRevision() const = 0;

	/**
	 * Get the history associated with this text instance.
	 */
	virtual const FTextHistory& GetTextHistory() const = 0;

	/**
	 * Get a mutable reference to the history associated with this text instance (used when loading/saving text).
	 */
	virtual FTextHistory& GetMutableTextHistory() = 0;
};
