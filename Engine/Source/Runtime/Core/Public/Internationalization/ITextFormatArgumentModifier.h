// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Crc.h"
#include "Containers/UnrealString.h"
#include "Internationalization/CulturePointer.h"

class FFormatArgumentValue;
struct FPrivateTextFormatArguments;

/**
 * Private type used to pass low-level format argument information through custom format argument modifiers without exposing the inner workings.
 */
struct FPrivateTextFormatArguments;

/**
 * Type used as a string literal by the text formatter.
 * It is a case sensitive string that can hold onto a string either by pointer (in which case the data being pointed to must outlive this object), or by taking a copy (stored as an FString internally).
 * @note The buffer is *not* guaranteed to be null terminated, so always test the length!
 */
class FTextFormatString
{
public:
	/** Construct an empty string */
	FTextFormatString()
		: StringPtr(TEXT(""))
		, StringLen(0)
		, InternalString()
	{
	}

	/** Construct from the given string (steals the value) */
	FTextFormatString(FString InStr)
		: StringPtr(nullptr)
		, StringLen(0)
		, InternalString(MoveTemp(InStr))
	{
		StringPtr = *InternalString;
		StringLen = InternalString.Len();
	}

	/** Construct from the given string (takes a copy, expected to be null terminated) */
	FTextFormatString(const TCHAR* InStr)
		: StringPtr(nullptr)
		, StringLen(0)
		, InternalString(InStr)
	{
		StringPtr = *InternalString;
		StringLen = InternalString.Len();
	}

	/** Construct from the given string (takes a reference, expected to be null terminated) */
	static FTextFormatString MakeReference(const TCHAR* InStr)
	{
		return FTextFormatString(InStr, FCString::Strlen(InStr));
	}

	/** Construct from the given pointer and size (takes a sub-string reference, doesn't have to be null terminated) */
	static FTextFormatString MakeReference(const TCHAR* InStr, const int32 InLen)
	{
		return FTextFormatString(InStr, InLen);
	}

	FTextFormatString(const FTextFormatString& Other)
		: StringPtr(Other.StringPtr)
		, StringLen(Other.StringLen)
		, InternalString(Other.InternalString)
	{
		if (Other.StringPtr == *Other.InternalString)
		{
			StringPtr = *InternalString;
		}
	}

	FTextFormatString(FTextFormatString&& Other)
		: StringPtr(Other.StringPtr)
		, StringLen(Other.StringLen)
		, InternalString(MoveTemp(Other.InternalString))
	{
	}

	FTextFormatString& operator=(const FTextFormatString& Other)
	{
		if (this != &Other)
		{
			StringPtr = Other.StringPtr;
			StringLen = Other.StringLen;
			InternalString = Other.InternalString;

			if (Other.StringPtr == *Other.InternalString)
			{
				StringPtr = *InternalString;
			}
		}
		return *this;
	}

	FTextFormatString& operator=(FTextFormatString&& Other)
	{
		if (this != &Other)
		{
			StringPtr = Other.StringPtr;
			StringLen = Other.StringLen;
			InternalString = MoveTemp(Other.InternalString);
		}
		return *this;
	}

	friend inline uint32 GetTypeHash(const FTextFormatString& InStr)
	{
		return FCrc::MemCrc32(InStr.StringPtr, sizeof(TCHAR) * InStr.StringLen);
	}

	/** The start of the string */
	const TCHAR* StringPtr;

	/** The length of the string */
	int32 StringLen;

private:
	/** Internal copy if constructed from a string */
	FString InternalString;

	/** Construct from the given pointer and size (takes a sub-string reference, doesn't have to be null terminated) */
	FTextFormatString(const TCHAR* InStr, const int32 InLen)
		: StringPtr(InStr)
		, StringLen(InLen)
		, InternalString()
	{
	}
};

inline bool operator==(const FTextFormatString& LHS, const FTextFormatString& RHS)
{
	return LHS.StringLen == RHS.StringLen 
		&& FCString::Strncmp(LHS.StringPtr, RHS.StringPtr, LHS.StringLen) == 0;
}

inline bool operator!=(const FTextFormatString& LHS, const FTextFormatString& RHS)
{
	return LHS.StringLen != RHS.StringLen
		|| FCString::Strncmp(LHS.StringPtr, RHS.StringPtr, LHS.StringLen) != 0;
}

inline bool operator<(const FTextFormatString& LHS, const FTextFormatString& RHS)
{
	if (LHS.StringLen < RHS.StringLen)
	{
		return true;
	}
	if (LHS.StringLen > RHS.StringLen)
	{
		return false;
	}
	return FCString::Strncmp(LHS.StringPtr, RHS.StringPtr, LHS.StringLen) < 0;
}

/**
 * Interface for a format argument modifier.
 */
class ITextFormatArgumentModifier
{
public:
	/** Virtual destructor */
	virtual ~ITextFormatArgumentModifier() = default;

	/** Validate the argument modifier is valid based on the rules of the given culture */
	virtual bool Validate(const FCultureRef& InCulture, TArray<FString>& OutValidationErrors) const = 0;

	/** Given the argument, evaluate the result and append it to OutResult */
	virtual void Evaluate(const FFormatArgumentValue& InValue, const FPrivateTextFormatArguments& InFormatArgs, FString& OutResult) const = 0;

	/** Get any argument names that are used by this argument modifier (for cases where the modifier itself uses format strings) */
	virtual void GetFormatArgumentNames(TArray<FString>& OutArgumentNames) const = 0;

	/** Quickly estimate the length of text that this argument modifier will likely inject into the string when evaluated */
	virtual void EstimateLength(int32& OutLength, bool& OutUsesFormatArgs) const = 0;

protected:
	/** Utility helper to parse out a list of key->value pair arguments. The keys are assumed to only contain valid identifier characters, and the values may be optionally quoted (parsed strings are sub-string references to within the source args string) */
	CORE_API static bool ParseKeyValueArgs(const FTextFormatString& InArgsString, TSortedMap<FTextFormatString, FTextFormatString>& OutArgKeyValues, const TCHAR InValueSeparator = TEXT('='), const TCHAR InArgSeparator = TEXT(','));

	/** Utility helper to parse out a list of value arguments. The values may be optionally quoted (parsed strings are sub-string references to within the source args string) */
	CORE_API static bool ParseValueArgs(const FTextFormatString& InArgsString, TArray<FTextFormatString>& OutArgValues, const TCHAR InArgSeparator = TEXT(','));
};
