// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Serialization/StructuredArchive.h"

class FString;
class FArchive;
class FName;

/**
 * Interface to an object that generates a localized string.
 */
class ITextGenerator
{
public:
	virtual ~ITextGenerator() = default;

	/**
	 * Produces the display string. This can be called multiple times if the language changes.
	 */
	virtual FString BuildLocalizedDisplayString() const = 0;

	/**
	 * Produces the display string for the invariant culture.
	 */
	virtual FString BuildInvariantDisplayString() const = 0;

public: // Serialization
	/**
	 * Gets the type ID of this generator. The type ID is used to reconstruct this type for serialization and must be registered with FText::RegisterTextGeneratorFactory().
	 *
	 * @see FText::RegisterTextGeneratorFactory()
	 */
	virtual FName GetTypeID() const = 0;

	/**
	 * Serializes this generator.
	 */
	virtual void Serialize(FStructuredArchive::FRecord Record) = 0;
};
