// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Internationalization/CulturePointer.h"
#include "Internationalization/LocTesting.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/UniqueObj.h"
#include "UObject/NameTypes.h"

#define LOC_DEFINE_REGION

class FCulture;
class FICUInternationalization;
class FLegacyInternationalization;
class ICustomCulture;

class FInternationalization
{
	friend class FTextChronoFormatter;

public:
	static CORE_API FInternationalization& Get();

	/** Checks to see that an internationalization instance exists, and has been initialized. Usually you would use Get(), however this may be used to work out whether TearDown() has been called when cleaning up on shutdown. */
	static CORE_API bool IsAvailable();

	static CORE_API void TearDown();

	/**
	 * Struct that can be used to capture a snapshot of the active culture state in a way that can be re-applied losslessly.
	 * Mostly used during automation testing.
	 */
	struct FCultureStateSnapshot
	{
		FString Language;
		FString Locale;
		TArray<TTuple<FName, FString>> AssetGroups;
	};

	/**
	 * Set the current culture by name.
	 * @note This function is a sledgehammer, and will set both the language and locale, as well as clear out any asset group cultures that may be set.
	 * @note SetCurrentCulture should be avoided in Core/Engine code as it may stomp over Editor/Game user-settings.
	 */
	CORE_API bool SetCurrentCulture(const FString& InCultureName);

	/**
	 * Get the current culture.
	 * @note This function exists for legacy API parity with SetCurrentCulture and is equivalent to GetCurrentLanguage. It should *never* be used in internal localization/internationalization code!
	 */
	FCultureRef GetCurrentCulture() const
	{
		return CurrentLanguage.ToSharedRef();
	}

	/**
	 * Set *only* the current language (for localization) by name.
	 * @note Unless you're doing something advanced, you likely want SetCurrentLanguageAndLocale or SetCurrentCulture instead.
	 */
	CORE_API bool SetCurrentLanguage(const FString& InCultureName);

	/**
	 * Get the current language (for localization).
	 */
	FCultureRef GetCurrentLanguage() const
	{
		return CurrentLanguage.ToSharedRef();
	}

	/**
	 * Set *only* the current locale (for internationalization) by name.
	 * @note Unless you're doing something advanced, you likely want SetCurrentLanguageAndLocale or SetCurrentCulture instead.
	 */
	CORE_API bool SetCurrentLocale(const FString& InCultureName);

	/**
	 * Get the current locale (for internationalization).
	 */
	FCultureRef GetCurrentLocale() const
	{
		return CurrentLocale.ToSharedRef();
	}

	/**
	 * Set the current language (for localization) and locale (for internationalization) by name.
	 */
	CORE_API bool SetCurrentLanguageAndLocale(const FString& InCultureName);

	/**
	 * Set the given asset group category culture by name.
	 */
	CORE_API bool SetCurrentAssetGroupCulture(const FName& InAssetGroupName, const FString& InCultureName);

	/**
	 * Get the given asset group category culture.
	 * @note Returns the current language if the group category doesn't have a culture override.
	 */
	CORE_API FCultureRef GetCurrentAssetGroupCulture(const FName& InAssetGroupName) const;

	/**
	 * Clear the given asset group category culture.
	 */
	CORE_API void ClearCurrentAssetGroupCulture(const FName& InAssetGroupName);

	/**
	 * Get the culture corresponding to the given name.
	 * @return The culture for the given name, or null if it couldn't be found.
	 */
	CORE_API FCulturePtr GetCulture(const FString& InCultureName);

	/**
	 * Get the default culture specified by the OS.
	 * @note This function exists for legacy API parity with GetCurrentCulture and is equivalent to GetDefaultLanguage. It should *never* be used in internal localization/internationalization code!
	 */
	FCultureRef GetDefaultCulture() const
	{
		return DefaultLanguage.ToSharedRef();
	}

	/**
	 * Get the default language specified by the OS.
	 */
	FCultureRef GetDefaultLanguage() const
	{
		return DefaultLanguage.ToSharedRef();
	}

	/**
	 * Get the default locale specified by the OS.
	 */
	FCultureRef GetDefaultLocale() const
	{
		return DefaultLocale.ToSharedRef();
	}

	/**
	 * Get the invariant culture that can be used when you don't care about localization/internationalization.
	 */
	FCultureRef GetInvariantCulture() const
	{
		return InvariantCulture.ToSharedRef();
	}

	/**
	 * Get the current cultures in use, optionally including the current language, locale, and any asset groups.
	 */
	CORE_API TArray<FCultureRef> GetCurrentCultures(const bool bIncludeLanguage, const bool bIncludeLocale, const bool bIncludeAssetGroups) const;

	/**
	 * Backup the current culture state to the given snapshot struct.
	 */
	CORE_API void BackupCultureState(FCultureStateSnapshot& OutSnapshot) const;

	/**
	 * Restore a previous culture state from the given snapshot struct.
	 */
	CORE_API void RestoreCultureState(const FCultureStateSnapshot& InSnapshot);

	bool IsInitialized() const {return bIsInitialized;}

	/** Load and cache the data needed for every culture we know about (this is usually done per-culture as required) */
	CORE_API void LoadAllCultureData();

	/** Add a new custom culture */
	CORE_API void AddCustomCulture(const TSharedRef<ICustomCulture>& InCustomCulture);

	/** Get a new custom culture from its name */
	CORE_API FCulturePtr GetCustomCulture(const FString& InCultureName) const;

	/** Has the given culture been remapped in this build? */
	CORE_API bool IsCultureRemapped(const FString& Name, FString* OutMappedCulture);

	/** Is the given culture enabled or disabled in this build? */
	CORE_API bool IsCultureAllowed(const FString& Name);

	/** Refresh the display names of the cached cultures */
	CORE_API void RefreshCultureDisplayNames(const TArray<FString>& InPrioritizedDisplayCultureNames);

	/** Refresh any config data that has been cached */
	CORE_API void RefreshCachedConfigData();

#if ENABLE_LOC_TESTING
	static CORE_API FString& Leetify(FString& SourceString);
#endif

	CORE_API void GetCultureNames(TArray<FString>& CultureNames) const;

	CORE_API TArray<FString> GetPrioritizedCultureNames(const FString& Name);

	/** Given some culture names, populate a list of cultures that are available to be used. If bIncludeDerivedCultures, include cultures that are derived from those we passed. */
	CORE_API TArray<FCultureRef> GetAvailableCultures(const TArray<FString>& InCultureNames, const bool bIncludeDerivedCultures);

	/** Broadcasts whenever the current culture changes */
	DECLARE_EVENT(FInternationalization, FCultureChangedEvent)
	FCultureChangedEvent& OnCultureChanged() { return CultureChangedEvent; }

private:
	FInternationalization();
	~FInternationalization();
	friend class FLazySingleton;

	void BroadcastCultureChanged() { CultureChangedEvent.Broadcast(); }

	void Initialize();
	void Terminate();

private:
	bool bIsInitialized = false;

	FCultureChangedEvent CultureChangedEvent;

#if UE_ENABLE_ICU
	friend class FICUInternationalization;
	typedef FICUInternationalization FImplementation;
#else
	friend class FLegacyInternationalization;
	typedef FLegacyInternationalization FImplementation;
#endif
	TUniqueObj<FImplementation> Implementation;

	/**
	 * The currently active language (for localization).
	 */
	FCulturePtr CurrentLanguage;

	/**
	 * The currently active locale (for internationalization).
	 */
	FCulturePtr CurrentLocale;

	/**
	 * The currently active asset group cultures (for package localization).
	 * @note This is deliberately an array for performance reasons (we expect to have a very small number of groups).
	 */
	TArray<TTuple<FName, FCulturePtr>> CurrentAssetGroupCultures;

	/**
	 * The default language specified by the OS.
	 */
	FCulturePtr DefaultLanguage;

	/**
	 * The default locale specified by the OS.
	 */
	FCulturePtr DefaultLocale;

	/**
	 * An invariant culture that can be used when you don't care about localization/internationalization.
	 */
	FCulturePtr InvariantCulture;

	/**
	 * Custom cultures registered via AddCustomCulture.
	 */
	TArray<FCultureRef> CustomCultures;
};

namespace UE::Private::LocGen
{
	inline FCulturePtr GetCultureByName(const TCHAR* InCulture)
	{
		return (InCulture && *InCulture)
			? FInternationalization::Get().GetCulture(InCulture)
			: nullptr;
	}
}

/** The global namespace that must be defined/undefined to wrap uses of the NS-prefixed macros below */
#undef LOCTEXT_NAMESPACE

/**
 * Creates an FText. All parameters must be string literals. All literals will be passed through the localization system.
 * The global LOCTEXT_NAMESPACE macro must be first set to a string literal to specify this localization key's namespace.
 */
#define LOCTEXT(InKey, InTextLiteral) FText::AsLocalizable_Advanced_LocText(TEXT(LOCTEXT_NAMESPACE), TEXT(InKey), TEXT(InTextLiteral))

/**
 * Creates an FText. All parameters must be string literals. All literals will be passed through the localization system.
 */
#define NSLOCTEXT(InNamespace, InKey, InTextLiteral) FText::AsLocalizable_Advanced_LocText(TEXT(InNamespace), TEXT(InKey), TEXT(InTextLiteral))

#if WITH_EDITORONLY_DATA
/**
 * Creates an FText with developer notes to provide context to translators. All parameters must be string literals. All literals will be passed through the localization system.
 * The global LOCTEXT_NAMESPACE macro must be first set to a string literal to specify this localization key's namespace.
 * Developer notes are stripped out of non-editor builds
 */
#define NOTELOCTEXT(InKey, InTextLiteral, InDevNotes) FText::AsLocalizable_Advanced_LocText(TEXT(LOCTEXT_NAMESPACE), TEXT(InKey), TEXT(InTextLiteral), TEXTVIEW(InDevNotes))

/**
 * Creates an FText with developer notes to provide context to translators. All parameters must be string literals. All literals will be passed through the localization system.
 * Developer notes are stripped out of non-editor builds
 */
#define NSNOTELOCTEXT(InNamespace, InKey, InTextLiteral, InDevNotes) FText::AsLocalizable_Advanced_LocText(TEXT(InNamespace), TEXT(InKey), TEXT(InTextLiteral), TEXTVIEW(InDevNotes))
#else
#define NOTELOCTEXT(InKey, InTextLiteral, InDevNotes) FText::AsLocalizable_Advanced_LocText(TEXT(LOCTEXT_NAMESPACE), TEXT(InKey), TEXT(InTextLiteral))
#define NSNOTELOCTEXT(InNamespace, InKey, InTextLiteral, InDevNotes) FText::AsLocalizable_Advanced_LocText(TEXT(InNamespace), TEXT(InKey), TEXT(InTextLiteral))
#endif
/**
 * Creates a culture invariant FText from the given string literal.
 */
#define INVTEXT(InTextLiteral) FText::AsCultureInvariant(TEXT(InTextLiteral))

/**
 * Generate an FText representation of the given number (alias for FText::AsNumber).
 * This macro exists to allow UHT to parse C++ default FText arguments in UFunctions (as this macro matches the syntax used by FTextStringHelper when exporting/importing stringified FText) and should not be used generally.
 * @param InNum			The number to generate the FText from.
 * @param InCulture		The culture code to use, or an empty string to use the active locale.
 * @param InOpts		Custom formatting options specified as chained setter functions of FNumberFormattingOptions (eg, SetAlwaysSign(true).SetUseGrouping(false)).
 */
#define LOCGEN_NUMBER(InNum, InCulture) FText::AsNumber(InNum, nullptr, UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))
#define LOCGEN_NUMBER_GROUPED(InNum, InCulture) FText::AsNumber(InNum, &FNumberFormattingOptions::DefaultWithGrouping(), UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))
#define LOCGEN_NUMBER_UNGROUPED(InNum, InCulture) FText::AsNumber(InNum, &FNumberFormattingOptions::DefaultNoGrouping(), UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))
#define LOCGEN_NUMBER_CUSTOM(InNum, InOpts, InCulture) FText::AsNumber(InNum, &FNumberFormattingOptions().InOpts, UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))

/**
 * Generate an FText representation of the given number as a percentage (alias for FText::AsPercent).
 * This macro exists to allow UHT to parse C++ default FText arguments in UFunctions (as this macro matches the syntax used by FTextStringHelper when exporting/importing stringified FText) and should not be used generally.
 * @param InNum			The number to generate the FText from.
 * @param InCulture		The culture code to use, or an empty string to use the active locale.
 * @param InOpts		Custom formatting options specified as chained setter functions of FNumberFormattingOptions (eg, SetAlwaysSign(true).SetUseGrouping(false)).
 */
#define LOCGEN_PERCENT(InNum, InCulture) FText::AsPercent(InNum, nullptr, UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))
#define LOCGEN_PERCENT_GROUPED(InNum, InCulture) FText::AsPercent(InNum, &FNumberFormattingOptions::DefaultWithGrouping(), UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))
#define LOCGEN_PERCENT_UNGROUPED(InNum, InCulture) FText::AsPercent(InNum, &FNumberFormattingOptions::DefaultNoGrouping(), UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))
#define LOCGEN_PERCENT_CUSTOM(InNum, InOpts, InCulture) FText::AsPercent(InNum, &FNumberFormattingOptions().InOpts, UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))

/**
 * Generate an FText representation of the given number as a currency (alias for FText::AsCurrencyBase).
 * This macro exists to allow UHT to parse C++ default FText arguments in UFunctions (as this macro matches the syntax used by FTextStringHelper when exporting/importing stringified FText) and should not be used generally.
 * @param InNum			The number to generate the FText from, specified in the smallest fractional value of the currency being used.
 * @param InCurrency	The currency code (eg, USD, GBP, EUR).
 * @param InCulture		The culture code to use, or an empty string to use the active locale.
 */
#define LOCGEN_CURRENCY(InNum, InCurrency, InCulture) FText::AsCurrencyBase(InNum, TEXT(InCurrency), UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))

/**
 * Generate an FText representation of the given timestamp as a date (alias for FText::AsDate).
 * This macro exists to allow UHT to parse C++ default FText arguments in UFunctions (as this macro matches the syntax used by FTextStringHelper when exporting/importing stringified FText) and should not be used generally.
 * @param InUnixTime	The Unix timestamp to generate the FText from.
 * @param InDateStyle	The style to use for the date.
 * @param InTimeZone	The timezone to display the timestamp in.
 * @param InCulture		The culture code to use, or an empty string to use the active locale.
 */
#define LOCGEN_DATE_UTC(InUnixTime, InDateStyle, InTimeZone, InCulture) FText::AsDate(FDateTime::FromUnixTimestamp(InUnixTime), InDateStyle, TEXT(InTimeZone), UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))
#define LOCGEN_DATE_LOCAL(InUnixTime, InDateStyle, InCulture) FText::AsDate(FDateTime::FromUnixTimestamp(InUnixTime), InDateStyle, FText::GetInvariantTimeZone(), UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))

/**
 * Generate an FText representation of the given timestamp as a time (alias for FText::AsTime).
 * This macro exists to allow UHT to parse C++ default FText arguments in UFunctions (as this macro matches the syntax used by FTextStringHelper when exporting/importing stringified FText) and should not be used generally.
 * @param InUnixTime	The Unix timestamp to generate the FText from.
 * @param InTimeStyle	The style to use for the time.
 * @param InTimeZone	The timezone to display the timestamp in.
 * @param InCulture		The culture code to use, or an empty string to use the active locale.
 */
#define LOCGEN_TIME_UTC(InUnixTime, InTimeStyle, InTimeZone, InCulture) FText::AsTime(FDateTime::FromUnixTimestamp(InUnixTime), InTimeStyle, TEXT(InTimeZone), UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))
#define LOCGEN_TIME_LOCAL(InUnixTime, InTimeStyle, InCulture) FText::AsTime(FDateTime::FromUnixTimestamp(InUnixTime), InTimeStyle, FText::GetInvariantTimeZone(), UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))

/**
 * Generate an FText representation of the given timestamp as a date and time (alias for FText::AsDateTime).
 * This macro exists to allow UHT to parse C++ default FText arguments in UFunctions (as this macro matches the syntax used by FTextStringHelper when exporting/importing stringified FText) and should not be used generally.
 * @param InUnixTime	The Unix timestamp to generate the FText from.
 * @param InDateStyle	The style to use for the date.
 * @param InTimeStyle	The style to use for the time.
 * @param InTimeZone	The timezone to display the timestamp in.
 * @param InCulture		The culture code to use, or an empty string to use the active locale.
 */
#define LOCGEN_DATETIME_UTC(InUnixTime, InDateStyle, InTimeStyle, InTimeZone, InCulture) FText::AsDateTime(FDateTime::FromUnixTimestamp(InUnixTime), InDateStyle, InTimeStyle, TEXT(InTimeZone), UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))
#define LOCGEN_DATETIME_LOCAL(InUnixTime, InDateStyle, InTimeStyle, InCulture) FText::AsDateTime(FDateTime::FromUnixTimestamp(InUnixTime), InDateStyle, InTimeStyle, FText::GetInvariantTimeZone(), UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))

 /**
  * Generate an FText representation of the given timestamp as a date and time (alias for FText::AsDateTime).
  * This macro exists to allow UHT to parse C++ default FText arguments in UFunctions (as this macro matches the syntax used by FTextStringHelper when exporting/importing stringified FText) and should not be used generally.
  * @param InUnixTime	The Unix timestamp to generate the FText from.
  * @param InPattern	The custom strftime-like pattern for the date/time string (see FDateTime::ToFormattedString).
  * @param InTimeZone	The timezone to display the timestamp in.
  * @param InCulture		The culture code to use, or an empty string to use the active locale.
  */
#define LOCGEN_DATETIME_CUSTOM_UTC(InUnixTime, InPattern, InTimeZone, InCulture) FText::AsDateTime(FDateTime::FromUnixTimestamp(InUnixTime), TEXT(InPattern), TEXT(InTimeZone), UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))
#define LOCGEN_DATETIME_CUSTOM_LOCAL(InUnixTime, InPattern, InCulture) FText::AsDateTime(FDateTime::FromUnixTimestamp(InUnixTime), TEXT(InPattern), FText::GetInvariantTimeZone(), UE::Private::LocGen::GetCultureByName(TEXT(InCulture)))

/**
 * Generate an FText representation of the given FText when transformed into upper-case (alias for FText::ToUpper).
 * This macro exists to allow UHT to parse C++ default FText arguments in UFunctions (as this macro matches the syntax used by FTextStringHelper when exporting/importing stringified FText) and should not be used generally.
 */
#define LOCGEN_TOUPPER(InText) (InText).ToUpper()

/**
 * Generate an FText representation of the given FText when transformed into lower-case (alias for FText::ToLower).
 * This macro exists to allow UHT to parse C++ default FText arguments in UFunctions (as this macro matches the syntax used by FTextStringHelper when exporting/importing stringified FText) and should not be used generally.
 */
#define LOCGEN_TOLOWER(InText) (InText).ToLower()

/**
 * Generate an FText representation of the given format pattern with the ordered arguments inserted into it (alias for FText::FormatOrdered).
 * This macro exists to allow UHT to parse C++ default FText arguments in UFunctions (as this macro matches the syntax used by FTextStringHelper when exporting/importing stringified FText) and should not be used generally.
 */
#define LOCGEN_FORMAT_ORDERED(InPattern, ...) FText::FormatOrdered(InPattern, __VA_ARGS__)

/**
 * Generate an FText representation of the given format pattern with the named arguments inserted into it (alias for FText::FormatNamed).
 * This macro exists to allow UHT to parse C++ default FText arguments in UFunctions (as this macro matches the syntax used by FTextStringHelper when exporting/importing stringified FText) and should not be used generally.
 */
#define LOCGEN_FORMAT_NAMED(InPattern, ...) FText::FormatNamed(InPattern, __VA_ARGS__)

#undef LOC_DEFINE_REGION
