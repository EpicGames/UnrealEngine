// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Internationalization/InternationalizationManifest.h"
#include "Internationalization/LocKeyFuncs.h"
#include "Misc/Crc.h"
#include "Templates/SharedPointer.h"

class FLocMetadataObject;

class FArchiveEntry
{
public:
	CORE_API FArchiveEntry(const FLocKey& InNamespace, const FLocKey& InKey, const FLocItem& InSource, const FLocItem& InTranslation, TSharedPtr<FLocMetadataObject> InKeyMetadataObj = nullptr, bool IsOptional = false);

	const FLocKey Namespace;
	const FLocKey Key;
	const FLocItem Source;
	FLocItem Translation;
	bool bIsOptional;
	TSharedPtr<FLocMetadataObject> KeyMetadataObj;
};

typedef TMultiMap< FLocKey, TSharedRef< FArchiveEntry > > FArchiveEntryByLocKeyContainer;
typedef TMultiMap< FString, TSharedRef< FArchiveEntry >, FDefaultSetAllocator, FLocKeyMultiMapFuncs< TSharedRef< FArchiveEntry > > > FArchiveEntryByStringContainer;

class FInternationalizationArchive 
{
public:
	enum class EFormatVersion : uint8
	{
		Initial = 0,
		EscapeFixes,
		AddedKeys,

		LatestPlusOne,
		Latest = LatestPlusOne - 1
	};

	FInternationalizationArchive()
		: FormatVersion(EFormatVersion::Latest)
	{
	}

	CORE_API bool AddEntry(const FLocKey& Namespace, const FLocKey& Key, const FLocItem& Source, const FLocItem& Translation, const TSharedPtr<FLocMetadataObject> KeyMetadataObj, const bool bOptional);
	CORE_API bool AddEntry(const TSharedRef<FArchiveEntry>& InEntry);

	CORE_API void UpdateEntry(const TSharedRef<FArchiveEntry>& OldEntry, const TSharedRef<FArchiveEntry>& NewEntry);

	CORE_API bool SetTranslation(const FLocKey& Namespace, const FLocKey& Key, const FLocItem& Source, const FLocItem& Translation, const TSharedPtr<FLocMetadataObject> KeyMetadataObj);

	CORE_API TSharedPtr<FArchiveEntry> FindEntryByKey(const FLocKey& Namespace, const FLocKey& Key, const TSharedPtr<FLocMetadataObject> KeyMetadataObj) const;

	FArchiveEntryByLocKeyContainer::TConstIterator GetEntriesByKeyIterator() const
	{
		return EntriesByKey.CreateConstIterator();
	}

	int32 GetNumEntriesByKey() const
	{
		return EntriesByKey.Num();
	}

	FArchiveEntryByStringContainer::TConstIterator GetEntriesBySourceTextIterator() const
	{
		return EntriesBySourceText.CreateConstIterator();
	}

	int32 GetNumEntriesBySourceText() const
	{
		return EntriesBySourceText.Num();
	}

	void SetFormatVersion(const EFormatVersion Version)
	{
		FormatVersion = Version;
	}

	EFormatVersion GetFormatVersion() const
	{
		return FormatVersion;
	}

private:
	EFormatVersion FormatVersion;
	FArchiveEntryByStringContainer EntriesBySourceText;
	FArchiveEntryByLocKeyContainer EntriesByKey;
};
