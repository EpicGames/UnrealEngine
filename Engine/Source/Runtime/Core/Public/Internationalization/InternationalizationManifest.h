// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Internationalization/LocKeyFuncs.h"
#include "Misc/Crc.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

class FLocMetadataObject;

struct FManifestContext
{
public:
	FManifestContext()
		: Key()
		, SourceLocation()
		, PlatformName()
		, bIsOptional(false)
	{
	}

	explicit FManifestContext(const FLocKey& InKey)
		: Key(InKey)
		, SourceLocation()
		, PlatformName()
		, bIsOptional(false)
	{
	}

	/** Copy ctor */
	CORE_API FManifestContext(const FManifestContext& Other);

	CORE_API FManifestContext& operator=(const FManifestContext& Other);
	CORE_API bool operator==(const FManifestContext& Other) const;
	inline bool operator!=(const FManifestContext& Other) const { return !(*this == Other); }
	CORE_API bool operator<(const FManifestContext& Other) const;

public:
	FLocKey Key;
	FString SourceLocation;
	FName PlatformName;
	bool bIsOptional;
	FString DevNotes;

	TSharedPtr<FLocMetadataObject> InfoMetadataObj;
	TSharedPtr<FLocMetadataObject> KeyMetadataObj;
};


struct FLocItem
{
public:
	FLocItem()
		: Text()
		, MetadataObj(nullptr)
	{
	}

	explicit FLocItem(FString InSourceText)
		: Text(MoveTemp(InSourceText))
		, MetadataObj(nullptr)
	{
	}

	FLocItem(FString InSourceText, TSharedPtr<FLocMetadataObject> InMetadataObj)
		: Text(MoveTemp(InSourceText))
		, MetadataObj(MoveTemp(InMetadataObj))
	{
	}

	/** Copy ctor */
	CORE_API FLocItem(const FLocItem& Other);

	CORE_API FLocItem& operator=(const FLocItem& Other);
	CORE_API bool operator==(const FLocItem& Other) const;
	inline bool operator!=(const FLocItem& Other) const { return !(*this == Other); }
	CORE_API bool operator<(const FLocItem& Other) const;

	/** Similar functionality to == operator but ensures everything matches(ex. ignores COMPARISON_MODIFIER_PREFIX on metadata). */
	CORE_API bool IsExactMatch(const FLocItem& Other) const;

public:
	FString Text;
	TSharedPtr<FLocMetadataObject> MetadataObj;
};


class FManifestEntry
{
public:
	FManifestEntry(const FLocKey& InNamespace, const FLocItem& InSource)
		: Namespace(InNamespace)
		, Source(InSource)
		, Contexts()
	{
	}

	CORE_API const FManifestContext* FindContext(const FLocKey& ContextKey, const TSharedPtr<FLocMetadataObject>& KeyMetadata = nullptr) const;
	CORE_API const FManifestContext* FindContextByKey(const FLocKey& ContextKey) const;
	CORE_API void MergeContextPlatformInfo(const FManifestContext& InContext);

	const FLocKey Namespace;
	const FLocItem Source;
	TArray<FManifestContext> Contexts;
};


typedef TMultiMap< FLocKey, TSharedRef< FManifestEntry > > FManifestEntryByLocKeyContainer;
typedef TMultiMap< FString, TSharedRef< FManifestEntry >, FDefaultSetAllocator, FLocKeyMultiMapFuncs< TSharedRef< FManifestEntry > > > FManifestEntryByStringContainer;


class FInternationalizationManifest 
{
public:
	enum class EFormatVersion : uint8
	{
		Initial = 0,
		EscapeFixes,

		LatestPlusOne,
		Latest = LatestPlusOne - 1
	};

	//Default constructor
	FInternationalizationManifest()
		: FormatVersion(EFormatVersion::Latest)
	{
	}

	/**
	* Adds a manifest entry.
	*
	* @return Returns true if add was successful or a matching entry already exists, false is only returned in the case where a duplicate context was found with different text.
	*/
	CORE_API bool AddSource(const FLocKey& Namespace, const FLocItem& Source, const FManifestContext& Context);

	CORE_API void UpdateEntry(const TSharedRef<FManifestEntry>& OldEntry, TSharedRef<FManifestEntry>& NewEntry);

	CORE_API TSharedPtr<FManifestEntry> FindEntryBySource(const FLocKey& Namespace, const FLocItem& Source) const;

	CORE_API TSharedPtr<FManifestEntry> FindEntryByContext(const FLocKey& Namespace, const FManifestContext& Context) const;

	CORE_API TSharedPtr<FManifestEntry> FindEntryByKey(const FLocKey& Namespace, const FLocKey& Key, const FString* SourceText = nullptr) const;

	CORE_API void ClearEntries();

	FManifestEntryByLocKeyContainer::TConstIterator GetEntriesByKeyIterator() const
	{
		return EntriesByKey.CreateConstIterator();
	}

	int32 GetNumEntriesByKey() const
	{
		return EntriesByKey.Num();
	}

	FManifestEntryByStringContainer::TConstIterator GetEntriesBySourceTextIterator() const
	{
		return EntriesBySourceText.CreateConstIterator();
	}

	TArray<TPair<FString, TSharedRef<FManifestEntry>>> GetEntriesBySourceTextAsArray() const
	{
		return EntriesBySourceText.Array();
	}

	int32 GetNumEntriesBySourceText() const
	{
		return EntriesBySourceText.Num();
	}

	void SetFormatVersion(const EFormatVersion Version)
	{
		FormatVersion = Version;
	}

	EFormatVersion GetFormatVersion() const
	{
		return FormatVersion;
	}

private:
	EFormatVersion FormatVersion;
	FManifestEntryByStringContainer EntriesBySourceText;
	FManifestEntryByLocKeyContainer EntriesByKey;
};
