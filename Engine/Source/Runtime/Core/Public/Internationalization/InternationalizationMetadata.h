// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/SharedPointer.h"

class FArchive;
class FLocMetadataObject;

/**
 * Represents all the types a LocMetadata Value can be.
 */
enum class ELocMetadataType : int32
{
	None,
	Boolean,
	String,
	Array,
	Object,
};

class FLocMetadataObject;

/**
 * A Metadata Value is a structure that can be a number of types.
 */
class FLocMetadataValue
{
public:
	CORE_API virtual ~FLocMetadataValue() = 0;

	virtual FString ToString() const = 0;

	/** Returns this value as a string, throwing an error if this is not a Metadata String */
	virtual FString AsString() {ErrorMessage(TEXT("String")); return FString();}

	/** Returns this value as a boolean, throwing an error if this is not an Metadata Bool */
	virtual bool AsBool() {ErrorMessage(TEXT("Boolean")); return false;}

	/** Returns this value as an array, throwing an error if this is not an Metadata Array */
	virtual TArray< TSharedPtr<FLocMetadataValue> > AsArray() {ErrorMessage(TEXT("Array")); return TArray< TSharedPtr<FLocMetadataValue> >();}

	/** Returns this value as an object, throwing an error if this is not an Metadata Object */
	virtual TSharedPtr<FLocMetadataObject> AsObject() {ErrorMessage(TEXT("Object")); return TSharedPtr<FLocMetadataObject>();}

	virtual TSharedRef<FLocMetadataValue> Clone() const = 0;

	virtual ELocMetadataType GetType() const = 0;

	bool operator==( const FLocMetadataValue& Other ) const { return ( (GetType() == Other.GetType()) && EqualTo( Other ) ); }
	bool operator<( const FLocMetadataValue& Other ) const { return (GetType() == Other.GetType()) ? LessThan( Other ) : (GetType() < Other.GetType()); }

protected:
	FLocMetadataValue() {}

	virtual bool EqualTo( const FLocMetadataValue& Other ) const = 0;
	virtual bool LessThan( const FLocMetadataValue& Other ) const = 0;

	virtual FString GetTypeString() const = 0;

	CORE_API void ErrorMessage(const FString& InType);

private:
	CORE_API FLocMetadataValue( const FLocMetadataValue& );
	CORE_API FLocMetadataValue& operator=( const FLocMetadataValue& );
};


/**
 * A LocMetadata Object is a structure holding an unordered set of name/value pairs.
 */
class FLocMetadataObject
{
public:
	FLocMetadataObject()
		: Values()
	{
	}

	/** Copy ctor */
	CORE_API FLocMetadataObject( const FLocMetadataObject& Other );
	
	template<ELocMetadataType LocMetadataType>
	TSharedPtr<FLocMetadataValue> GetField( const FString& FieldName )
	{
		const TSharedPtr<FLocMetadataValue>* Field = Values.Find(FieldName);
		if ( ensureMsgf(Field && Field->IsValid(), TEXT("Field %s was not found."), *FieldName) )
		{
			if ( ensureMsgf( (*Field)->GetType() == LocMetadataType, TEXT("Field %s is of the wrong type."), *FieldName) )
			{
				return (*Field);
			}
		}

		return TSharedPtr<FLocMetadataValue>();
	}

	/** Checks to see if the FieldName exists in the object. */
	bool HasField( const FString& FieldName)
	{
		const TSharedPtr<FLocMetadataValue>* Field = Values.Find(FieldName);
		if(Field && Field->IsValid())
		{
			return true;
		}

		return false;
	}
	
	/** Checks to see if the FieldName exists in the object, and has the specified type. */
	template<ELocMetadataType LocMetadataType>
	bool HasTypedField(const FString& FieldName)
	{
		const TSharedPtr<FLocMetadataValue>* Field = Values.Find(FieldName);
		if(Field && Field->IsValid() && ((*Field)->GetType() == LocMetadataType))
		{
			return true;
		}

		return false;
	}

	CORE_API void SetField( const FString& FieldName, const TSharedPtr<FLocMetadataValue>& Value );

	CORE_API void RemoveField(const FString& FieldName);

	/** Get the field named FieldName as a string. Ensures that the field is present and is of type LocMetadata string. */
	CORE_API FString GetStringField(const FString& FieldName);

	/** Add a field named FieldName with value of StringValue */
	CORE_API void SetStringField( const FString& FieldName, const FString& StringValue );

	/** Get the field named FieldName as a boolean. Ensures that the field is present and is of type LocMetadata boolean. */
	CORE_API bool GetBoolField(const FString& FieldName);

	/** Set a boolean field named FieldName and value of InValue */
	CORE_API void SetBoolField( const FString& FieldName, bool InValue );

	/** Get the field named FieldName as an array. Ensures that the field is present and is of type LocMetadata Array. */
	CORE_API TArray< TSharedPtr<FLocMetadataValue> > GetArrayField(const FString& FieldName);

	/** Set an array field named FieldName and value of Array */
	CORE_API void SetArrayField( const FString& FieldName, const TArray< TSharedPtr<FLocMetadataValue> >& Array );

	/** Get the field named FieldName as a LocMetadata object. Ensures that the field is present and is a LocMetadata object*/
	CORE_API TSharedPtr<FLocMetadataObject> GetObjectField(const FString& FieldName);

	/** Set an ObjectField named FieldName and value of LocMetadataObject */
	CORE_API void SetObjectField( const FString& FieldName, const TSharedPtr<FLocMetadataObject>& LocMetadataObject );

	CORE_API FLocMetadataObject& operator=( const FLocMetadataObject& Other );

	CORE_API bool operator==(const FLocMetadataObject& Other) const;
	CORE_API bool operator<( const FLocMetadataObject& Other ) const;

	/** Similar functionality to == operator but ensures everything matches(ignores COMPARISON_MODIFIER_PREFIX). */
	CORE_API bool IsExactMatch( const FLocMetadataObject& Other ) const;

	static CORE_API bool IsMetadataExactMatch( const FLocMetadataObject* const MetadataA, const FLocMetadataObject* const MetadataB );

	CORE_API FString ToString() const;

	CORE_API friend FArchive& operator<<(FArchive& Archive, FLocMetadataObject& Object);
	CORE_API friend void operator<<(FStructuredArchive::FSlot Slot, FLocMetadataObject& Object);

public:
	/** Stores the name/value pairs for the metadata object */
	TMap< FString, TSharedPtr<FLocMetadataValue> > Values;

	/** Special reserved character.  When encountered as a prefix metadata name(the key in the Values map), it will adjust the way comparisons are done. */
	static CORE_API const TCHAR* COMPARISON_MODIFIER_PREFIX;
};

/** A LocMetadata String Value. */
class FLocMetadataValueString : public FLocMetadataValue
{
public:
	FLocMetadataValueString(const FString& InString) : Value(InString) {}
	CORE_API FLocMetadataValueString(FStructuredArchive::FSlot Slot);
	static CORE_API void Serialize(FLocMetadataValueString& Value, FStructuredArchive::FSlot Slot);

	virtual FString AsString() override {return Value;}

	CORE_API virtual TSharedRef<FLocMetadataValue> Clone() const override;
	void SetString( const FString& InString ) { Value = InString; }

	virtual FString ToString() const override {return Value;}

	virtual ELocMetadataType GetType() const override {return ELocMetadataType::String;}

protected:

	FString Value;

	CORE_API virtual bool EqualTo( const FLocMetadataValue& Other ) const override;
	CORE_API virtual bool LessThan( const FLocMetadataValue& Other ) const override;

	virtual FString GetTypeString() const override {return TEXT("String");}
};


/** A LocMetadata Boolean Value. */
class FLocMetadataValueBoolean : public FLocMetadataValue
{
public:
	FLocMetadataValueBoolean(bool InBool) : Value(InBool) {}
	CORE_API FLocMetadataValueBoolean(FStructuredArchive::FSlot Slot);
	static CORE_API void Serialize(FLocMetadataValueBoolean& Value, FStructuredArchive::FSlot Slot);

	virtual bool AsBool() override {return Value;}

	CORE_API virtual TSharedRef<FLocMetadataValue> Clone() const override;
	
	virtual FString ToString() const override {return Value ? TEXT("true") : TEXT("false");}

	virtual ELocMetadataType GetType() const override {return ELocMetadataType::Boolean;}

protected:
	bool Value;

	CORE_API virtual bool EqualTo( const FLocMetadataValue& Other ) const override;
	CORE_API virtual bool LessThan( const FLocMetadataValue& Other ) const override;

	virtual FString GetTypeString() const override {return TEXT("Boolean");}
};

/** A LocMetadata Array Value. */
class FLocMetadataValueArray : public FLocMetadataValue
{
public:
	FLocMetadataValueArray(const TArray< TSharedPtr<FLocMetadataValue> >& InArray) : Value(InArray) {}
	CORE_API FLocMetadataValueArray(FStructuredArchive::FSlot Slot);
	static CORE_API void Serialize(FLocMetadataValueArray& Value, FStructuredArchive::FSlot Slot);

	virtual TArray< TSharedPtr<FLocMetadataValue> > AsArray() override {return Value;}

	CORE_API virtual TSharedRef<FLocMetadataValue> Clone() const override;
	
	CORE_API virtual FString ToString() const override;

	virtual ELocMetadataType GetType() const override {return ELocMetadataType::Array;}

protected:
	TArray< TSharedPtr<FLocMetadataValue> > Value;

	CORE_API virtual bool EqualTo( const FLocMetadataValue& Other ) const override;
	CORE_API virtual bool LessThan( const FLocMetadataValue& Other ) const override;

	virtual FString GetTypeString() const override {return TEXT("Array");}
};

/** A LocMetadata Object Value. */
class FLocMetadataValueObject : public FLocMetadataValue
{
public:
	FLocMetadataValueObject(TSharedPtr<FLocMetadataObject> InObject) : Value(InObject) {}
	CORE_API FLocMetadataValueObject(FStructuredArchive::FSlot Slot);
	static CORE_API void Serialize(FLocMetadataValueObject& Value, FStructuredArchive::FSlot Slot);

	virtual TSharedPtr<FLocMetadataObject> AsObject() override {return Value;}

	CORE_API virtual TSharedRef<FLocMetadataValue> Clone() const override;
	
	CORE_API virtual FString ToString() const override;

	virtual ELocMetadataType GetType() const override {return ELocMetadataType::Object;}

protected:
	TSharedPtr<FLocMetadataObject> Value;

	CORE_API virtual bool EqualTo( const FLocMetadataValue& Other ) const override;
	CORE_API virtual bool LessThan( const FLocMetadataValue& Other ) const override;

	virtual FString GetTypeString() const override {return TEXT("Object");}
};
