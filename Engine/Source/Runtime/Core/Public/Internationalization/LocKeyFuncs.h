// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/Set.h"
#include "Containers/Map.h"

/** Case sensitive immutable hashed string used as a localization key */
class FLocKey
{
public:
	FLocKey()
		: Str()
		, Hash(0)
	{
	}

	FLocKey(const TCHAR* InStr)
		: Str(InStr)
		, Hash(ProduceHash(Str))
	{
	}

	FLocKey(const FString& InStr)
		: Str(InStr)
		, Hash(ProduceHash(Str))
	{
	}

	FLocKey(FString&& InStr)
		: Str(MoveTemp(InStr))
		, Hash(ProduceHash(Str))
	{
	}

	FLocKey(const FLocKey& InOther)
		: Str(InOther.Str)
		, Hash(InOther.Hash)
	{
	}

	FLocKey(FLocKey&& InOther)
		: Str(MoveTemp(InOther.Str))
		, Hash(InOther.Hash)
	{
		InOther.Hash = 0;
	}

	FLocKey& operator=(const FLocKey& InOther)
	{
		if (this != &InOther)
		{
			Str = InOther.Str;
			Hash = InOther.Hash;
		}

		return *this;
	}

	FLocKey& operator=(FLocKey&& InOther)
	{
		if (this != &InOther)
		{
			Str = MoveTemp(InOther.Str);
			Hash = InOther.Hash;

			InOther.Hash = 0;
		}

		return *this;
	}

	UE_FORCEINLINE_HINT bool operator==(const FLocKey& Other) const
	{
		return Hash == Other.Hash && Compare(Other) == 0;
	}

	UE_FORCEINLINE_HINT bool operator!=(const FLocKey& Other) const
	{
		return Hash != Other.Hash || Compare(Other) != 0;
	}

	UE_FORCEINLINE_HINT bool operator<(const FLocKey& Other) const
	{
		return Compare(Other) < 0;
	}

	UE_FORCEINLINE_HINT bool operator<=(const FLocKey& Other) const
	{
		return Compare(Other) <= 0;
	}

	UE_FORCEINLINE_HINT bool operator>(const FLocKey& Other) const
	{
		return Compare(Other) > 0;
	}

	UE_FORCEINLINE_HINT bool operator>=(const FLocKey& Other) const
	{
		return Compare(Other) >= 0;
	}

	friend inline uint32 GetTypeHash(const FLocKey& Id)
	{
		return Id.Hash;
	}

	UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return Str.IsEmpty();
	}

	UE_FORCEINLINE_HINT bool Equals(const FLocKey& Other) const
	{
		return *this == Other;
	}

	UE_FORCEINLINE_HINT int32 Compare(const FLocKey& Other) const
	{
		return FCString::Strcmp(*Str, *Other.Str);
	}

	UE_FORCEINLINE_HINT const FString& GetString() const
	{
		return Str;
	}

	static UE_FORCEINLINE_HINT uint32 ProduceHash(const FString& InStr, const uint32 InBaseHash = 0)
	{
		return FCrc::StrCrc32<TCHAR>(*InStr, InBaseHash);
	}

private:
	/** String representation of this LocKey */
	FString Str;

	/** Hash representation of this LocKey */
	uint32 Hash;
};

/** Case sensitive hashing function for TSet */
struct FLocKeySetFuncs : BaseKeyFuncs<FString, FString>
{
	static UE_FORCEINLINE_HINT const FString& GetSetKey(const FString& Element)
	{
		return Element;
	}
	static UE_FORCEINLINE_HINT bool Matches(const FString& A, const FString& B)
	{
		return A.Equals(B, ESearchCase::CaseSensitive);
	}
	static UE_FORCEINLINE_HINT uint32 GetKeyHash(const FString& Key)
	{
		return FLocKey::ProduceHash(Key);
	}
};

/** Case sensitive hashing function for TMap */
template <typename ValueType>
struct FLocKeyMapFuncs : BaseKeyFuncs<ValueType, FString, /*bInAllowDuplicateKeys*/false>
{
	static UE_FORCEINLINE_HINT const FString& GetSetKey(const TPair<FString, ValueType>& Element)
	{
		return Element.Key;
	}
	static UE_FORCEINLINE_HINT bool Matches(const FString& A, const FString& B)
	{
		return A.Equals(B, ESearchCase::CaseSensitive);
	}
	static UE_FORCEINLINE_HINT uint32 GetKeyHash(const FString& Key)
	{
		return FLocKey::ProduceHash(Key);
	}
};

/** Case sensitive hashing function for TMultiMap */
template <typename ValueType>
struct FLocKeyMultiMapFuncs : BaseKeyFuncs<ValueType, FString, /*bInAllowDuplicateKeys*/true>
{
	static UE_FORCEINLINE_HINT const FString& GetSetKey(const TPair<FString, ValueType>& Element)
	{
		return Element.Key;
	}
	static UE_FORCEINLINE_HINT bool Matches(const FString& A, const FString& B)
	{
		return A.Equals(B, ESearchCase::CaseSensitive);
	}
	static UE_FORCEINLINE_HINT uint32 GetKeyHash(const FString& Key)
	{
		return FLocKey::ProduceHash(Key);
	}
};

/** Case sensitive comparison function for TSortedMap */
struct FLocKeySortedMapLess
{
	UE_FORCEINLINE_HINT bool operator()(const FString& A, const FString& B) const
	{
		return A.Compare(B, ESearchCase::CaseSensitive) < 0;
	}
};
