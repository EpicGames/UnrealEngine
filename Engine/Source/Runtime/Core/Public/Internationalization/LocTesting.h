// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

#ifndef ENABLE_LOC_TESTING
	#define ENABLE_LOC_TESTING (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT || UE_BUILD_TEST)
#endif

static_assert(!ENABLE_LOC_TESTING || !UE_BUILD_SHIPPING, "ENABLE_LOC_TESTING can never be enabled in shipping builds");
