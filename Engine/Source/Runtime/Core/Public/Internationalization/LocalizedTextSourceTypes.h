// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/UEOps.h"
#include "Templates/RefCounting.h"
#include "Templates/Requires.h"
#include "Templates/SharedPointer.h"
#include "UObject/CoreReflectedTypeAccessors.h"

#include <type_traits>

/**
 * Categories of localized text.
 * @note This enum is mirrored in NoExportTypes.h for UHT.
 */
enum class ELocalizedTextSourceCategory : uint8
{
	Game,
	Engine,
	Editor,
};

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<ELocalizedTextSourceCategory>();
#endif

/**
 * Result codes from calling QueryLocalizedResourceResult.
 */
enum class EQueryLocalizedResourceResult : uint8
{
	/** Indicates the query found a matching entry and added its result */
	Found,
	/** Indicates that the query failed to find a matching entry */
	NotFound,
	/** Indicates that the query failed as this text source doesn't support queries */
	NotImplemented,
};

/**
 * Load flags used in localization initialization.
 */
enum class ELocalizationLoadFlags : uint8
{
	/** Load no data */
	None = 0,

	/** Load native data */
	Native = 1<<0,

	/** Load editor localization data */
	Editor = 1<<1,

	/** Load game localization data */
	Game = 1<<2,

	/** Load engine localization data */
	Engine = 1<<3,

	/** Load additional (eg, plugin) localization data */
	Additional = 1<<4,

	/** Force localized game data to be loaded, even when running in the editor */
	ForceLocalizedGame = 1<<5,

	/**
	 * Skip updating any entries that already exist in the live table
	 * @note Not useful when performing a full update, but has utility when patching in new untrusted localization data (eg, loading UGC localization data over the base localization data)
	 */
	SkipExisting = 1<<6,
};
ENUM_CLASS_FLAGS(ELocalizationLoadFlags);

/**
 * Pre-defined priorities for ILocalizedTextSource.
 */
struct ELocalizedTextSourcePriority
{
	enum Enum
	{
		Lowest = -1000,
		Low = -100,
		Normal = 0,
		High = 100,
		Highest = 1000,
	};
};

namespace UE::Text::Private
{

class FRefCountedDisplayString : public TRefCountingMixin<FRefCountedDisplayString>
{
public:
	explicit FRefCountedDisplayString(FString&& InDisplayString)
		: DisplayString(MoveTemp(InDisplayString))
	{
	}

	[[nodiscard]] FString& Private_GetDisplayString()
	{
		return DisplayString;
	}

private:
	FString DisplayString;
};

/**
 * Wrapper to give TRefCountPtr a minimal TSharedRef/TSharedPtr interface for backwards compatibility with code that was already using FTextDisplayStringRef/FTextDisplayStringPtr
 */
template <typename ObjectType>
class TDisplayStringPtrBase
{
	static_assert(TPointerIsConvertibleFromTo<ObjectType, const FString>::Value, "TDisplayStringPtrBase can only be constructed with FString types");

public:
	TDisplayStringPtrBase() = default;

	explicit TDisplayStringPtrBase(const TRefCountPtr<FRefCountedDisplayString>& InDisplayStringPtr)
		: DisplayStringPtr(InDisplayStringPtr)
	{
	}

	[[nodiscard]] explicit operator bool() const
	{
		return IsValid();
	}

	[[nodiscard]] bool IsValid() const
	{
		return DisplayStringPtr.IsValid();
	}

	[[nodiscard]] ObjectType& operator*() const
	{
		return GetDisplayString();
	}

	[[nodiscard]] ObjectType* operator->() const
	{
		return &GetDisplayString();
	}

	[[nodiscard]] const TRefCountPtr<FRefCountedDisplayString>& Private_GetDisplayStringPtr() const
	{
		return DisplayStringPtr;
	}

	template <typename OtherObjectType>
	[[nodiscard]] bool UEOpEquals(const TDisplayStringPtrBase<OtherObjectType>& Rhs) const
	{
		return DisplayStringPtr == Rhs.DisplayStringPtr;
	}

protected:
	[[nodiscard]] FString& GetDisplayString() const
	{
		check(IsValid());
		return DisplayStringPtr.GetReference()->Private_GetDisplayString();
	}

	TRefCountPtr<FRefCountedDisplayString> DisplayStringPtr;
};

/**
 * Wrapper to give TRefCountPtr a minimal TSharedRef interface for backwards compatibility with code that was already using FTextDisplayStringRef
 */
template <typename ObjectType>
class TDisplayStringRef : public TDisplayStringPtrBase<ObjectType>
{
public:
	TDisplayStringRef() = default;

	explicit TDisplayStringRef(const TRefCountPtr<FRefCountedDisplayString>& InDisplayStringPtr)
		: TDisplayStringPtrBase<ObjectType>(InDisplayStringPtr)
	{
		check(this->IsValid());
	}

	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	TDisplayStringRef(const TDisplayStringRef<OtherType>& InOther)
		: TDisplayStringPtrBase<ObjectType>(InOther.Private_GetDisplayStringPtr())
	{
	}

	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	TDisplayStringRef& operator=(const TDisplayStringRef<OtherType>& InOther)
	{
		if (this->DisplayStringPtr != InOther.Private_GetDisplayStringPtr())
		{
			this->DisplayStringPtr = InOther.Private_GetDisplayStringPtr();
		}
		return *this;
	}

	[[nodiscard]] ObjectType& Get() const
	{
		return this->GetDisplayString();
	}
};

/**
 * Wrapper to give TRefCountPtr a minimal TSharedPtr interface for backwards compatibility with code that was already using FTextDisplayStringPtr
 */
template <typename ObjectType>
class TDisplayStringPtr : public TDisplayStringPtrBase<ObjectType>
{
public:
	TDisplayStringPtr() = default;

	TDisplayStringPtr(TYPE_OF_NULLPTR)
		: TDisplayStringPtr()
	{
	}

	explicit TDisplayStringPtr(const TRefCountPtr<FRefCountedDisplayString>& InDisplayStringPtr)
		: TDisplayStringPtrBase<ObjectType>(InDisplayStringPtr)
	{
	}

	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	TDisplayStringPtr(const TDisplayStringPtr<OtherType>& InOther)
		: TDisplayStringPtrBase<ObjectType>(InOther.Private_GetDisplayStringPtr())
	{
	}

	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	TDisplayStringPtr(const TDisplayStringRef<OtherType>& InOther)
		: TDisplayStringPtrBase<ObjectType>(InOther.Private_GetDisplayStringPtr())
	{
	}

	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	TDisplayStringPtr& operator=(const TDisplayStringPtr<OtherType>& InOther)
	{
		if (this->DisplayStringPtr != InOther.Private_GetDisplayStringPtr())
		{
			this->DisplayStringPtr = InOther.Private_GetDisplayStringPtr();
		}
		return *this;
	}

	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	TDisplayStringPtr& operator=(const TDisplayStringRef<OtherType>& InOther)
	{
		if (this->DisplayStringPtr != InOther.Private_GetDisplayStringPtr())
		{
			this->DisplayStringPtr = InOther.Private_GetDisplayStringPtr();
		}
		return *this;
	}

	[[nodiscard]] ObjectType* Get() const
	{
		return this->IsValid()
			? &this->GetDisplayString()
			: nullptr;
	}

	void Reset()
	{
		this->DisplayStringPtr = TRefCountPtr<FRefCountedDisplayString>();
	}

	[[nodiscard]] TDisplayStringRef<ObjectType> ToSharedRef() const
	{
		check(this->IsValid());
		return TDisplayStringRef<ObjectType>(this->DisplayStringPtr);
	}
};

} // namespace UE::Text::Private

template <typename ObjectType>
[[nodiscard]] uint32 GetTypeHash(const UE::Text::Private::TDisplayStringRef<ObjectType>& A)
{
	return ::PointerHash(&A.Get());
}

template <typename ObjectType>
[[nodiscard]] uint32 GetTypeHash(const UE::Text::Private::TDisplayStringPtr<ObjectType>& A)
{
	return ::PointerHash(A.Get());
}

using FTextDisplayStringRef = UE::Text::Private::TDisplayStringRef<FString>;
using FTextDisplayStringPtr = UE::Text::Private::TDisplayStringPtr<FString>;
using FTextConstDisplayStringRef = UE::Text::Private::TDisplayStringRef<const FString>;
using FTextConstDisplayStringPtr = UE::Text::Private::TDisplayStringPtr<const FString>;

[[nodiscard]] inline FTextDisplayStringRef MakeTextDisplayString(FString&& InDisplayString)
{
	return FTextDisplayStringRef(MakeRefCount<UE::Text::Private::FRefCountedDisplayString>(MoveTemp(InDisplayString)));
}
