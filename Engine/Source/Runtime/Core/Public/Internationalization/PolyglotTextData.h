// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Internationalization/LocalizedTextSourceTypes.h"
#include "Internationalization/Text.h"

/**
 * Polyglot data that may be registered to the text localization manager at runtime.
 * @note This struct is mirrored in NoExportTypes.h for UHT.
 */
struct FPolyglotTextData
{
	/** Needed to avoid member access issues in UHT generated code */
	friend struct Z_Construct_UScriptStruct_FPolyglotTextData_Statics;

public:
	/**
	 * Default constructor
	 */
	FPolyglotTextData() = default;

	/**
	 * Construct this polyglot data with an identity, optionally immediately providing the source text and native culture.
	 */
	CORE_API FPolyglotTextData(const ELocalizedTextSourceCategory& InCategory, const FString& InNamespace, const FString& InKey, const FString& InNativeString, const FString& InNativeCulture = FString());

	/**
	 * Is this polyglot data valid and can be registered with the text localization manager?
	 */
	CORE_API bool IsValid(FText* OutFailureReason = nullptr) const;

	/**
	 * Set the category of this polyglot data.
	 * @note This affects when and how the data is loaded into the text localization manager.
	 */
	CORE_API void SetCategory(const ELocalizedTextSourceCategory InCategory);

	/**
	 * Get the category of this polyglot data.
	 */
	CORE_API ELocalizedTextSourceCategory GetCategory() const;

	/**
	 * Set the native culture of this polyglot data.
	 * @note This may be empty, and if empty, will be inferred from the native culture of the text category.
	 */
	CORE_API void SetNativeCulture(const FString& InNativeCulture);

	/**
	 * Get the native culture of this polyglot data.
	 */
	CORE_API const FString& GetNativeCulture() const;

	/**
	 * Resolve the native culture of this polyglot data, either using the native culture if specified, or inferring it from the category.
	 */
	CORE_API FString ResolveNativeCulture() const;

	/**
	 * Get the localized cultures of this polyglot data.
	 */
	CORE_API TArray<FString> GetLocalizedCultures() const;

	/**
	 * Set the identity of the text created from this polyglot data.
	 * @note The key must not be empty.
	 */
	CORE_API void SetIdentity(const FString& InNamespace, const FString& InKey);

	/**
	 * Get the identity of the text created from this polyglot data.
	 */
	CORE_API void GetIdentity(FString& OutNamespace, FString& OutKey) const;

	/**
	 * Get the namespace of the text created from this polyglot data.
	 */
	CORE_API const FString& GetNamespace() const;

	/**
	 * Get the key of the text created from this polyglot data.
	 */
	CORE_API const FString& GetKey() const;

	/**
	 * Set the native string of this polyglot data.
	 * @note The native string must not be empty.
	 */
	CORE_API void SetNativeString(const FString& InNativeString);

	/**
	 * Get the native string of this polyglot data.
	 */
	CORE_API const FString& GetNativeString() const;
	
	/**
	 * Add a localized string to this polyglot data.
	 * @note The native culture may also have a translation added via this function.
	 */
	CORE_API void AddLocalizedString(const FString& InCulture, const FString& InLocalizedString);

	/**
	 * Remove a localized string from this polyglot data.
	 */
	CORE_API void RemoveLocalizedString(const FString& InCulture);

	/**
	 * Get a localized string from this polyglot data.
	 */
	CORE_API bool GetLocalizedString(const FString& InCulture, FString& OutLocalizedString) const;

	/**
	 * Clear the localized strings from this polyglot data.
	 */
	CORE_API void ClearLocalizedStrings();

	/**
	 * Set whether this polyglot data is a minimal patch.
	 * @see bIsMinimalPatch.
	 */
	CORE_API void IsMinimalPatch(const bool InIsMinimalPatch);

	/**
	 * Get whether this polyglot data is a minimal patch.
	 * @see bIsMinimalPatch.
	 */
	CORE_API bool IsMinimalPatch() const;

	/**
	 * Get the text instance created from this polyglot data.
	 */
	CORE_API FText GetText() const;

private:
	/**
	 * Cache the text instance created from this polyglot data.
	 */
	CORE_API void CacheText(FText* OutFailureReason = nullptr);

	/**
	 * Clear the cache of the text instance created from this polyglot data.
	 */
	CORE_API void ClearCache();

	/**
	 * The category of this polyglot data.
	 * @note This affects when and how the data is loaded into the text localization manager.
	 */
	ELocalizedTextSourceCategory Category;

	/**
	 * The native culture of this polyglot data.
	 * @note This may be empty, and if empty, will be inferred from the native culture of the text category.
	 */
	FString NativeCulture;

	/**
	 * The namespace of the text created from this polyglot data.
	 * @note This may be empty.
	 */
	FString Namespace;

	/**
	 * The key of the text created from this polyglot data.
	 * @note This must not be empty.
	 */
	FString Key;

	/**
	 * The native string for this polyglot data.
	 * @note This must not be empty (it should be the same as the originally authored text you are trying to replace).
	 */
	FString NativeString;

	/**
	 * Mapping between a culture code and its localized string.
	 * @note The native culture may also have a translation in this map.
	 */
	TMap<FString, FString> LocalizedStrings;

	/**
	 * True if this polyglot data is a minimal patch, and that missing translations should be 
	 * ignored (falling back to any LocRes data) rather than falling back to the native string.
	 */
	bool bIsMinimalPatch = false;

	/**
	 * Transient cached text instance from registering this polyglot data with the text localization manager.
	 */
	FText CachedText;
};
