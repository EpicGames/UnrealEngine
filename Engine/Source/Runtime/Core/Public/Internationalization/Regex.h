// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"

class FRegexPatternImplementation;
class FRegexMatcherImplementation;

enum class ERegexPatternFlags
{
	None = 0,
	CaseInsensitive = (1 << 0),
};
ENUM_CLASS_FLAGS(ERegexPatternFlags);

/**
 * Implements a regular expression pattern.
 * @note DO NOT use this class as a file-level variable as its construction relies on the internationalization system being initialized!
 */
class FRegexPattern
{
	friend class FRegexMatcher;

public:
	CORE_API explicit FRegexPattern(const FString& SourceString, ERegexPatternFlags Flags = ERegexPatternFlags::None);

	FRegexPattern(const FRegexPattern&) = default;
	FRegexPattern& operator=(const FRegexPattern&) = default;

	FRegexPattern(FRegexPattern&&) = default;
	FRegexPattern& operator=(FRegexPattern&&) = default;

private:
	TSharedRef<FRegexPatternImplementation> Implementation;
};

/**
 * Implements a regular expression pattern matcher.
 * @note DO NOT use this class as a file-level variable as its construction relies on the internationalization system being initialized!
 */
class FRegexMatcher
{
public:
	CORE_API FRegexMatcher(const FRegexPattern& SourcePattern, const FString& Input);
	CORE_API FRegexMatcher(FRegexPattern&& SourcePattern, const FString& Input);

	FRegexMatcher(const FRegexMatcher&) = delete;
	FRegexMatcher& operator=(const FRegexMatcher&) = delete;

	FRegexMatcher(FRegexMatcher&&) = default;
	FRegexMatcher& operator=(FRegexMatcher&&) = default;

	CORE_API bool FindNext();

	CORE_API FString ReplaceFirst(FStringView Replacement);
	CORE_API FString ReplaceAll(FStringView Replacement);

	CORE_API int32 GetMatchBeginning();
	CORE_API int32 GetMatchEnding();

	CORE_API int32 GetCaptureGroupBeginning(const int32 Index);
	CORE_API int32 GetCaptureGroupEnding(const int32 Index);
	CORE_API FString GetCaptureGroup(const int32 Index);

	CORE_API int32 GetBeginLimit();
	CORE_API int32 GetEndLimit();
	CORE_API void SetLimits(const int32 BeginIndex, const int32 EndIndex);

private:
	FRegexPattern Pattern;
	TSharedRef<FRegexMatcherImplementation> Implementation;
};
