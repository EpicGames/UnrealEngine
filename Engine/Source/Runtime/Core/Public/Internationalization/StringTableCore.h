// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/LocKeyFuncs.h"
#include "Internationalization/LocalizedTextSourceTypes.h"
#include "Internationalization/StringTableCoreFwd.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextKey.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

#include <atomic>

class FStringTable;
class UStringTable;

CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogStringTable, Log, All);

/** Entry within a string table */
class FStringTableEntry
{
public:
	/** Create a new string table entry using the given data */
#if WITH_EDITORONLY_DATA
	static FStringTableEntryRef NewStringTableEntry(FStringTableConstRef InOwnerTable, FString InSourceString, FTextId InDisplayStringId, FString InDevNotes)
	{
		return MakeShared<FStringTableEntry, ESPMode::ThreadSafe>(MoveTemp(InOwnerTable), MoveTemp(InSourceString), MoveTemp(InDisplayStringId), MoveTemp(InDevNotes));
	}
#else
	static FStringTableEntryRef NewStringTableEntry(FStringTableConstRef InOwnerTable, FString InSourceString, FTextId InDisplayStringId)
	{
		return MakeShared<FStringTableEntry, ESPMode::ThreadSafe>(MoveTemp(InOwnerTable), MoveTemp(InSourceString), MoveTemp(InDisplayStringId));
	}
#endif

	/** Default constructor */
	CORE_API FStringTableEntry();

	/** Create a new string table entry using the given data */
#if WITH_EDITORONLY_DATA
	CORE_API FStringTableEntry(FStringTableConstRef InOwnerTable, FString InSourceString, FTextId InDisplayStringId, FString InDevNotes);
#else
	CORE_API FStringTableEntry(FStringTableConstRef InOwnerTable, FString InSourceString, FTextId InDisplayStringId);
#endif

	/** @return true if this entry is currently owned by a string table, false if it's been disowned (and should be re-cached) */
	CORE_API bool IsOwned() const;

	/** Disown this string table entry. This is used to notify external code that has cached this entry that it needs to re-cache it from the string table */
	CORE_API void Disown();

	/** Is this string table entry owned by the given string table? */
	CORE_API bool IsOwnedBy(const FStringTable& InStringTable) const;

	/** Get the source string of this string table entry */
	CORE_API const FString& GetSourceString() const;

	/** Get the display string of this string table entry */
	CORE_API FTextConstDisplayStringPtr GetDisplayString() const;

	/** Get the display string ID of this string table entry */
	CORE_API FTextId GetDisplayStringId() const;
	
#if WITH_EDITORONLY_DATA
	/** Get the translation context notes */
	const FString& GetDevNotes() const { return DevNotes; }
#endif

	/** Get the placeholder source string to use for string table entries that are missing */
	static CORE_API const FString& GetPlaceholderSourceString();

private:
	/** The string table that owns us (if any) */
	FStringTableConstWeakPtr OwnerTable;

	/** The source string of this entry */
	FString SourceString;

	/** The display string ID of this entry */
	FTextId DisplayStringId;
	
#if WITH_EDITORONLY_DATA
	/** Notes from the developer to provide additional context to translators */
	FString DevNotes;
#endif
};

/** String table implementation. Holds Key->SourceString pairs of text. */
class FStringTable : public TSharedFromThis<FStringTable, ESPMode::ThreadSafe>
{
public:
	/** Create a new string table */
	static FStringTableRef NewStringTable()
	{
		return MakeShared<FStringTable, ESPMode::ThreadSafe>();
	}

	/** Default constructor */
	CORE_API FStringTable();

	/** Destructor */
	CORE_API ~FStringTable();

	/** @return The asset that owns this string table instance (if any) */
	CORE_API UStringTable* GetOwnerAsset() const;

	/** Set the asset that owns this string table instance (if any) */
	CORE_API void SetOwnerAsset(UStringTable* InOwnerAsset);

	/** Has this string table been fully loaded yet? (used during asset loading) */
	CORE_API bool IsLoaded() const;

	/** Set whether this string table has been fully loaded */
	CORE_API void IsLoaded(const bool bInIsLoaded);

	/** Is this string table considered internal (ie, shouldn't appear in pickers) */
	CORE_API bool IsInternal() const;

	/** Set whether this string table is considered internal */
	CORE_API void IsInternal(const bool bInIsInternal);

	/** @return The namespace used by all entries in this string table */
	CORE_API FString GetNamespace() const;

	/** Set the namespace used by all entries in this string table */
	CORE_API void SetNamespace(const FTextKey& InNamespace);

	/** Get the source string used by the given entry (if any) */
	CORE_API bool GetSourceString(const FTextKey& InKey, FString& OutSourceString) const;
	
#if WITH_EDITORONLY_DATA
	/** Set the source string and dev notes used by the given entry (will replace any existing data for that entry) */
	CORE_API void SetSourceString(const FTextKey& InKey, const FString& InSourceString, const FString& InDevNotes);
#else
	/** Set the source string used by the given entry (will replace any existing data for that entry) */
	CORE_API void SetSourceString(const FTextKey& InKey, const FString& InSourceString);
#endif

	/** Remove the given entry (including its meta-data) */
	CORE_API void RemoveSourceString(const FTextKey& InKey);
	
#if WITH_EDITORONLY_DATA
	/** Enumerate all source strings in the table with DevNotes passed in as the third param. Return true from the enumerator to continue, or false to stop */
	CORE_API void EnumerateSourceStrings(const TFunctionRef<bool(const FString&, const FString&, const FString&)>& InEnumerator) const;
	CORE_API void EnumerateKeysAndSourceStrings(const TFunctionRef<bool(const FTextKey&, const FString&, const FString&)>& InEnumerator) const;
#endif

	/** Enumerate all source strings in the table. Return true from the enumerator to continue, or false to stop */
	CORE_API void EnumerateSourceStrings(const TFunctionRef<bool(const FString&, const FString&)>& InEnumerator) const;
	CORE_API void EnumerateKeysAndSourceStrings(const TFunctionRef<bool(const FTextKey&, const FString&)>& InEnumerator) const;

	/** Clear all entries from the table (including their meta-data) */
	CORE_API void ClearSourceStrings(const int32 InSlack = 0);

	/** Find the entry with the given key (if any) */
	CORE_API FStringTableEntryConstPtr FindEntry(const FTextKey& InKey) const;

	/** Given an entry, check to see if it exists in this table, and if so, get its key */
	CORE_API bool FindKey(const FStringTableEntryConstRef& InEntry, FString& OutKey) const;
	CORE_API bool FindKey(const FStringTableEntryConstRef& InEntry, FTextKey& OutKey) const;

	/** Get the meta-data with the given ID associated with the given entry, or an empty string if not found */
	CORE_API FString GetMetaData(const FTextKey& InKey, const FName InMetaDataId) const;

	/** Set the meta-data with the given ID associated with the given entry */
	CORE_API void SetMetaData(const FTextKey& InKey, const FName InMetaDataId, const FString& InMetaDataValue);

	/** Remove the meta-data with the given ID associated with the given entry */
	CORE_API void RemoveMetaData(const FTextKey& InKey, const FName InMetaDataId);

	/** Enumerate all meta-data associated with the given entry. Return true from the enumerator to continue, or false to stop */
	CORE_API void EnumerateMetaData(const FTextKey& InKey, const TFunctionRef<bool(FName, const FString&)>& InEnumerator) const;

	/** Remove all meta-data associated with the given entry */
	CORE_API void ClearMetaData(const FTextKey& InKey);

	/** Clear all meta-data from the table */
	CORE_API void ClearMetaData(const int32 InSlack = 0);

	/** Serialize this string table to/from an archive */
	CORE_API void Serialize(FArchive& Ar);

	/** Export the key, string, and meta-data information in this string table to a CSV file (does not export the namespace) */
	UE_DEPRECATED(5.8, "ExportStrings is deprecated in favor of ExportStringsToCSVFile")
	CORE_API bool ExportStrings(const FString& InFilename) const;

	/** Export the key, string, and meta-data information in this string table to a CSV string (does not export the namespace) */
	CORE_API FString ExportStringsToCSVString() const;

	/** Export the key, string, and meta-data information in this string table to a CSV file (does not export the namespace) */
	CORE_API bool ExportStringsToCSVFile(const FString& InFilename) const;

	/** Import key, string, and meta-data information from a CSV file to this string table (does not import the namespace) */
	UE_DEPRECATED(5.8, "ImportStrings is deprecated in favor of ImportStringsFromCSVFile")
	CORE_API bool ImportStrings(const FString& InFilename);

	/** Import key, string, and meta-data information from a CSV string to this string table (does not import the namespace) */
	CORE_API bool ImportStringsFromCSVString(const FString& InCSVString, const TCHAR* InErrorContext = TEXT("<unknown>"));

	/** Import key, string, and meta-data information from a CSV file to this string table (does not import the namespace) */
	CORE_API bool ImportStringsFromCSVFile(const FString& InFilename);

private:
	/** Pointer back to the asset that owns this table */
	UStringTable* OwnerAsset;

	/** True if this table has been fully loaded (used for assets) */
	bool bIsLoaded;

	/** True if this table is internal and shouldn't appear in pickers */
	bool bIsInternal;

	/** The namespace to use for all the strings in this table */
	FTextKey TableNamespace;

	/** Mapping between the text key and entry data for the strings within this table */
	TMap<FTextKey, FStringTableEntryPtr> KeysToEntries;

	/** Critical section preventing concurrent modification of KeysToEntries */
	mutable FTransactionallySafeCriticalSection KeyMappingCS;

	/** Mapping between the text key and its meta-data map */
	typedef TMap<FName, FString> FMetaDataMap;
	TMap<FTextKey, FMetaDataMap> KeysToMetaData;

	/** Critical section preventing concurrent modification of KeysToMetaData */
	mutable FTransactionallySafeCriticalSection KeysToMetaDataCS;
};

/** Interface to allow Core code to access String Table assets from the Engine */
class IStringTableEngineBridge
{
public:
	/**
	 * Scope object used to temporarily defer String Table find/load (eg, during module load).
	 */
	struct FScopedDeferFindOrLoad
	{
		FScopedDeferFindOrLoad()
		{
			++DeferFindOrLoad;
		}

		~FScopedDeferFindOrLoad()
		{
			--DeferFindOrLoad;
		}

		UE_NONCOPYABLE(FScopedDeferFindOrLoad);
	};

	/**
	 * Callback used when loading string table assets.
	 * @param The name of the table we were asked to load.
	 * @param The name of the table we actually loaded (may be different if redirected; will be empty if the load failed).
	 */
	typedef TFunction<void(FName, FName)> FLoadStringTableAssetCallback;

	/** 
	 * Check to see whether it is currently safe to attempt to find or load a string table asset.
	 * @return True if it is safe to attempt to find or load a string table asset, false otherwise.
	 */
	static bool CanFindOrLoadStringTableAsset()
	{
		return FInternationalization::IsAvailable()
			&& DeferFindOrLoad.load(std::memory_order_relaxed) <= 0
			&& (!InstancePtr || InstancePtr->CanFindOrLoadStringTableAssetImpl());
	}

	/**
	 * Load a string table asset by its name, potentially doing so asynchronously. 
	 * @note If the string table is already loaded, or loading is perform synchronously, then the callback will be called before this function returns.
	 * @return The async loading ID of the asset, or INDEX_NONE if no async loading was performed.
	 */
	static int32 LoadStringTableAsset(const FName InTableId, FLoadStringTableAssetCallback InLoadedCallback = FLoadStringTableAssetCallback())
	{
		check(CanFindOrLoadStringTableAsset());

		if (InstancePtr)
		{
			return InstancePtr->LoadStringTableAssetImpl(InTableId, InLoadedCallback);
		}

		// No bridge instance - just say it's already loaded
		if (InLoadedCallback)
		{
			InLoadedCallback(InTableId, InTableId);
		}
		return INDEX_NONE;
	}

	/**
	 * Fully load a string table asset by its name, synchronously.
	 * @note This should be used sparingly in places where it is definitely safe to perform a blocking load.
	 */
	static void FullyLoadStringTableAsset(FName& InOutTableId)
	{
		check(CanFindOrLoadStringTableAsset());

		if (InstancePtr)
		{
			return InstancePtr->FullyLoadStringTableAssetImpl(InOutTableId);
		}
	}

	/** Redirect string table asset by its name */
	static void RedirectStringTableAsset(FName& InOutTableId)
	{
		check(CanFindOrLoadStringTableAsset());

		if (InstancePtr)
		{
			InstancePtr->RedirectStringTableAssetImpl(InOutTableId);
		}
	}

	/** Collect a string table asset reference */
	static void CollectStringTableAssetReferences(FName& InOutTableId, FStructuredArchive::FSlot Slot)
	{
		if (InstancePtr)
		{
			InstancePtr->CollectStringTableAssetReferencesImpl(InOutTableId, Slot);
		}
	}

	/** Collect any searchable name data for a string table asset entry reference */
	static void CollectStringTableAssetSearchableName(const FName InTableId, const FTextKey InKey, const FArchive& Ar)
	{
		if (InstancePtr)
		{
			InstancePtr->CollectStringTableAssetSearchableNameImpl(InTableId, InKey, Ar);
		}
	}

	/** Is this string table from an asset? */
	static bool IsStringTableFromAsset(const FName InTableId)
	{
		return InstancePtr && InstancePtr->IsStringTableFromAssetImpl(InTableId);
	}

	/** Is this string table asset being replaced due to a hot-reload? */
	static bool IsStringTableAssetBeingReplaced(const UStringTable* InStringTableAsset)
	{
		return InstancePtr && InStringTableAsset && InstancePtr->IsStringTableAssetBeingReplacedImpl(InStringTableAsset);
	}

protected:
	virtual ~IStringTableEngineBridge() = default;

	virtual bool CanFindOrLoadStringTableAssetImpl() = 0;
	virtual int32 LoadStringTableAssetImpl(const FName InTableId, FLoadStringTableAssetCallback InLoadedCallback) = 0;
	virtual void FullyLoadStringTableAssetImpl(FName& InOutTableId) = 0;
	virtual void RedirectStringTableAssetImpl(FName& InOutTableId) = 0;
	virtual void CollectStringTableAssetReferencesImpl(FName& InOutTableId, FStructuredArchive::FSlot Slot) = 0;
	virtual void CollectStringTableAssetSearchableNameImpl(const FName InTableId, const FTextKey InKey, const FArchive& Ar) = 0;
	virtual bool IsStringTableFromAssetImpl(const FName InTableId) = 0;
	virtual bool IsStringTableAssetBeingReplacedImpl(const UStringTable* InStringTableAsset) = 0;

	/** Singleton instance, populated by the derived type */
	static CORE_API IStringTableEngineBridge* InstancePtr;

	/** Whether String Table find/load is currently deferred (eg, during module load) */
	static CORE_API std::atomic<int8> DeferFindOrLoad;
};

/** String table redirect utils */
struct FStringTableRedirects
{
	/** Initialize the string table redirects */
	static CORE_API void InitStringTableRedirects();

	/** Redirect a table ID */
	static CORE_API void RedirectTableId(FName& InOutTableId);

	/** Redirect a key */
	static CORE_API void RedirectKey(const FName InTableId, FTextKey& InOutKey);

	/** Redirect a table ID and key */
	static CORE_API void RedirectTableIdAndKey(FName& InOutTableId, FTextKey& InOutKey);
};
