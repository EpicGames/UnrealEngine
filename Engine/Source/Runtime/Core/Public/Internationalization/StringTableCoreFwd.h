// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

/** Loading policy to use with String Table assets */
enum class EStringTableLoadingPolicy : uint8
{
	/** Try and find the String Table, but do not attempt to load it */
	Find,
	/** Try and find the String Table, or attempt of load it if it cannot be found (note: the string table found may not be fully loaded) */
	FindOrLoad,
	/** Try and find the String Table, or attempt to load it if it cannot be found, or if it was found but not fully loaded (note: this should be used sparingly in places where it is definitely safe to perform a blocking load) */
	FindOrFullyLoad,
};

class UStringTable;

class FStringTableEntry;
typedef TSharedPtr<FStringTableEntry, ESPMode::ThreadSafe> FStringTableEntryPtr;
typedef TSharedRef<FStringTableEntry, ESPMode::ThreadSafe> FStringTableEntryRef;
typedef TWeakPtr<FStringTableEntry, ESPMode::ThreadSafe> FStringTableEntryWeakPtr;
typedef TSharedPtr<const FStringTableEntry, ESPMode::ThreadSafe> FStringTableEntryConstPtr;
typedef TSharedRef<const FStringTableEntry, ESPMode::ThreadSafe> FStringTableEntryConstRef;
typedef TWeakPtr<const FStringTableEntry, ESPMode::ThreadSafe> FStringTableEntryConstWeakPtr;

class FStringTable;
typedef TSharedPtr<FStringTable, ESPMode::ThreadSafe> FStringTablePtr;
typedef TSharedRef<FStringTable, ESPMode::ThreadSafe> FStringTableRef;
typedef TWeakPtr<FStringTable, ESPMode::ThreadSafe> FStringTableWeakPtr;
typedef TSharedPtr<const FStringTable, ESPMode::ThreadSafe> FStringTableConstPtr;
typedef TSharedRef<const FStringTable, ESPMode::ThreadSafe> FStringTableConstRef;
typedef TWeakPtr<const FStringTable, ESPMode::ThreadSafe> FStringTableConstWeakPtr;
