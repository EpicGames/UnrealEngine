// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"
#include "Internationalization/LocKeyFuncs.h"
#include "Internationalization/StringTableCoreFwd.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextKey.h"
#include "Internationalization/TextLocalizationManager.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "UObject/NameTypes.h"

class UStringTable;
template <typename FuncType> class TFunctionRef;

/** Singleton registry of known string table instances */
class FStringTableRegistry
{
public:
	/** Singleton accessor */
	static CORE_API FStringTableRegistry& Get();

	/** Register a string table with the given ID */
	CORE_API void RegisterStringTable(const FName InTableId, FStringTableRef InTable);

	/** Unregister a string table with the given ID, returning true if there was a registered string table with the given ID */
	CORE_API bool UnregisterStringTable(const FName InTableId);

	/** Try and find a string table with the given ID */
	CORE_API FStringTablePtr FindMutableStringTable(const FName InTableId) const;

	/** Try and find a string table with the given ID */
	CORE_API FStringTableConstPtr FindStringTable(const FName InTableId) const;

	/** Try and find a string table asset with the given ID */
	CORE_API UStringTable* FindStringTableAsset(const FName InTableId) const;

	/** Enumerate all registered string tables */
	CORE_API void EnumerateStringTables(const TFunctionRef<bool(const FName&, const FStringTableConstRef&)>& InEnumerator) const;

	/** Log a missing string table entry (will only log each missing entry once to avoid spam) */
	CORE_API void LogMissingStringTableEntry(const FName InTableId, const FTextKey& InKey);

	/** Internal function called by LOCTABLE_NEW to create and register a new FStringTable instance */
	CORE_API void Internal_NewLocTable(const FName InTableId, const FTextKey& InNamespace);

	/** Internal function called by LOCTABLE_FROMFILE_X to create and register a FStringTable instance that has been populated from a file */
	CORE_API void Internal_LocTableFromFile(const FName InTableId, const FTextKey& InNamespace, const FString& InFilePath, const FString& InRootPath);

	/** Internal function called by LOCTABLE_SETSTRING and LOCTABLE_SETNOTESTRING to set the entry denoted by the given key to the given source string, within the given string table (table must have been registered already) */
#if WITH_EDITORONLY_DATA
	CORE_API void Internal_SetLocTableEntry(const FName InTableId, const FTextKey& InKey, const FString& InSourceString, const FString& InDevNotes = TEXT(""));
#else
	CORE_API void Internal_SetLocTableEntry(const FName InTableId, const FTextKey& InKey, const FString& InSourceString);
#endif

	/** Internal function called by LOCTABLE_SETMETA to set meta-data for the entry denoted by the given key, within the given string table (table must have been registered already) */
	CORE_API void Internal_SetLocTableEntryMetaData(const FName InTableId, const FTextKey& InKey, const FName InMetaDataId, const FString& InMetaData);

	/** Internal function called by LOCTABLE to find the entry with by the given key within the given string table (redirects, will load assets if needed, and returns a dummy FText if not found) */
	CORE_API FText Internal_FindLocTableEntry(const FName InTableId, const FTextKey& InKey, const EStringTableLoadingPolicy InLoadingPolicy) const;

private:
	/** Private constructor - use singleton accessor */
	FStringTableRegistry();
	~FStringTableRegistry();

	/** Mapping from a table ID to a string table instance */
	TMap<FName, FStringTablePtr> RegisteredStringTables;

	/** Critical section preventing concurrent modification of RegisteredStringTables */
	mutable FTransactionallySafeCriticalSection RegisteredStringTablesCS;

	/** Mapping from a table ID to a set of keys that we've logged warnings for */
	typedef TSet<FTextKey> FLocKeySet;
	TMap<FName, FLocKeySet> LoggedMissingEntries;

	/** Critical section preventing concurrent modification of LoggedMissingEntries */
	mutable FTransactionallySafeCriticalSection LoggedMissingEntriesCS;

#if WITH_EDITOR
	/** Callback handler for a directory change notification */
	void OnDirectoryChanged(const TArray<struct FFileChangeData>& InFileChanges);

	/** Mapping of absolute CSV file paths to the table ID that imported them using Internal_LocTableFromFile */
	TMap<FString, FName> CSVFilesToWatch;

	/** Critical section preventing concurrent modification of CSVFilesToWatch */
	mutable FCriticalSection CSVFilesToWatchCS;

	/** Delegate handle watching the Engine directory */
	FDelegateHandle EngineDirectoryWatcherHandle;

	/** Delegate handle watching the Game directory */
	FDelegateHandle GameDirectoryWatcherHandle;
#endif // WITH_EDITOR
};

#define LOC_DEFINE_REGION

/** Creates and registers a new string table instance. */
#define LOCTABLE_NEW(ID, NAMESPACE) \
	FStringTableRegistry::Get().Internal_NewLocTable(TEXT(ID), TEXT(NAMESPACE))

/** Creates and registers a new string table instance, loading strings from the given file (the path is relative to the engine content directory). */
#define LOCTABLE_FROMFILE_ENGINE(ID, NAMESPACE, FILEPATH) \
	FStringTableRegistry::Get().Internal_LocTableFromFile(TEXT(ID), TEXT(NAMESPACE), TEXT(FILEPATH), FPaths::EngineContentDir())

/** Creates and registers a new string table instance, loading strings from the given file (the path is relative to the game content directory). */
#define LOCTABLE_FROMFILE_GAME(ID, NAMESPACE, FILEPATH) \
	FStringTableRegistry::Get().Internal_LocTableFromFile(TEXT(ID), TEXT(NAMESPACE), TEXT(FILEPATH), FPaths::ProjectContentDir())

/** Add a string table entry with the given key and source string. */
#define LOCTABLE_SETSTRING(ID, KEY, SRC) \
	FStringTableRegistry::Get().Internal_SetLocTableEntry(TEXT(ID), TEXT(KEY), TEXT(SRC))

#if WITH_EDITORONLY_DATA
/** Add a string table entry with the given key, source string, and developer translation context note. */
#define LOCTABLE_SETNOTESTRING(ID, KEY, SRC, NOTE) \
	FStringTableRegistry::Get().Internal_SetLocTableEntry(TEXT(ID), TEXT(KEY), TEXT(SRC), TEXT(NOTE))
#else
#define LOCTABLE_SETNOTESTRING(ID, KEY, SRC, NOTE) LOCTABLE_SETSTRING(ID, KEY, SRC)
#endif

/** Add a meta-data for the entry with the given key and source string. */
#define LOCTABLE_SETMETA(ID, KEY, METAID, META) \
	FStringTableRegistry::Get().Internal_SetLocTableEntryMetaData(TEXT(ID), TEXT(KEY), TEXT(METAID), TEXT(META))

/** Find a string table with the given ID, and try and find an entry within it using the given key. Returns a dummy FText if not found. */
#define LOCTABLE(ID, KEY) \
	FStringTableRegistry::Get().Internal_FindLocTableEntry(TEXT(ID), TEXT(KEY), EStringTableLoadingPolicy::FindOrLoad)

#undef LOC_DEFINE_REGION
