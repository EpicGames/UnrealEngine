// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AutoRTFM/ReturnFromOpenMode.h"
#include "CoreTypes.h"
#include "HAL/PlatformAtomics.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/UnrealTypeTraits.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Containers/SortedMap.h"
#include "Containers/EnumAsByte.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/TextKey.h"
#include "Internationalization/LocKeyFuncs.h"
#include "Internationalization/CulturePointer.h"
#include "Internationalization/TextComparison.h"
#include "Internationalization/TextLocalizationManager.h"
#include "Internationalization/StringTableCoreFwd.h"
#include "Internationalization/ITextData.h"
#include "Misc/Optional.h"
#include "Templates/UniquePtr.h"
#include "Templates/IsConstructible.h"
#include "Templates/Requires.h"

#include <type_traits>

class FText;
class FTextHistory;
class FTextFormatData;
class FFormatArgumentValue;
class FHistoricTextFormatData;
class FHistoricTextNumericData;
class FTextFormatPatternDefinition;
class ITextGenerator;

//DECLARE_CYCLE_STAT_EXTERN( TEXT("Format Text"), STAT_TextFormat, STATGROUP_Text, );

namespace ETextFlag
{
	enum Type
	{
		Transient = (1 << 0),
		CultureInvariant = (1 << 1),
		ConvertedProperty = (1 << 2),
		Immutable = (1 << 3),
		InitializedFromString = (1<<4),  // this ftext was initialized using FromString
	};
}

enum class ETextIdenticalModeFlags : uint8
{
	/** No special behavior */
	None = 0,

	/**
	 * Deep compare the text data.
	 *
	 * When set, two pieces of generated text (eg, from FText::Format, FText::AsNumber, FText::AsDate, FText::ToUpper, etc) 
	 * will test their internal data to see if they contain identical inputs (so would produce an identical output).
	 *
	 * When clear, no two separate pieces of generated text will ever compare as identical!
	 */
	DeepCompare = 1<<0,

	/**
	 * Compare invariant data lexically.
	 *
	 * When set, two pieces of invariant text (eg, from FText::AsCultureInvariant, FText::FromString, FText::FromName, or INVTEXT)
	 * will compare their display string data lexically to see if they are identical.
	 *
	 * When clear, no two separate pieces of invariant text will ever compare as identical!
	 */
	LexicalCompareInvariants = 1<<1,
};
ENUM_CLASS_FLAGS(ETextIdenticalModeFlags);

enum class ETextFormatFlags : uint8
{
	/** No special behavior */
	None = 0,

	/**
	 * Set to evaluate argument modifiers when formatting text
	 * Unset to print the literal argument modifier syntax into the result
	 */
	EvaluateArgumentModifiers = 1<<0,

	/** Default formatting flags */
	Default = EvaluateArgumentModifiers,
};
ENUM_CLASS_FLAGS(ETextFormatFlags);

enum class ETextPluralType : uint8
{
	Cardinal,
	Ordinal,
};

enum class ETextPluralForm : uint8
{
	Zero = 0,
	One,	// Singular
	Two,	// Dual
	Few,	// Paucal
	Many,	// Also used for fractions if they have a separate class
	Other,	// General plural form, also used if the language only has a single form
	Count,	// Number of entries in this enum
};

/** Redeclared in KismetTextLibrary for meta-data extraction purposes, be sure to update there as well */
enum class ETextGender : uint8
{
	Masculine,
	Feminine,
	Neuter,
	// Add new enum types at the end only! They are serialized by index.
};
CORE_API bool LexTryParseString(ETextGender& OutValue, const TCHAR* Buffer);
CORE_API void LexFromString(ETextGender& OutValue, const TCHAR* Buffer);
CORE_API const TCHAR* LexToString(ETextGender InValue);

/** Redeclared in KismetTextLibrary for meta-data extraction purposes, be sure to update there as well */
namespace EDateTimeStyle
{
	enum Type : int
	{
		Default,
		Short,
		Medium,
		Long,
		Full,
		Custom,	// Internal use only
		// Add new enum types at the end only! They are serialized by index.
	};
}
CORE_API bool LexTryParseString(EDateTimeStyle::Type& OutValue, const TCHAR* Buffer);
CORE_API void LexFromString(EDateTimeStyle::Type& OutValue, const TCHAR* Buffer);
CORE_API const TCHAR* LexToString(EDateTimeStyle::Type InValue);

/** Redeclared in KismetTextLibrary for meta-data extraction purposes, be sure to update there as well */
namespace EFormatArgumentType
{
	enum Type : int
	{
		Int,
		UInt,
		Float,
		Double,
		Text,
		Gender,
		// Add new enum types at the end only! They are serialized by index.
	};
}

typedef TSortedMap<FString, FFormatArgumentValue, FDefaultAllocator, FLocKeySortedMapLess> FFormatNamedArguments;
typedef TArray<FFormatArgumentValue> FFormatOrderedArguments;

typedef TSharedRef<FTextFormatPatternDefinition, ESPMode::ThreadSafe> FTextFormatPatternDefinitionRef;
typedef TSharedPtr<FTextFormatPatternDefinition, ESPMode::ThreadSafe> FTextFormatPatternDefinitionPtr;
typedef TSharedRef<const FTextFormatPatternDefinition, ESPMode::ThreadSafe> FTextFormatPatternDefinitionConstRef;
typedef TSharedPtr<const FTextFormatPatternDefinition, ESPMode::ThreadSafe> FTextFormatPatternDefinitionConstPtr;

/** Redeclared in KismetTextLibrary for meta-data extraction purposes, be sure to update there as well */
enum ERoundingMode : int
{
	/** Rounds to the nearest place, equidistant ties go to the value which is closest to an even value: 1.5 becomes 2, 0.5 becomes 0 */
	HalfToEven,
	/** Rounds to nearest place, equidistant ties go to the value which is further from zero: -0.5 becomes -1.0, 0.5 becomes 1.0 */
	HalfFromZero,
	/** Rounds to nearest place, equidistant ties go to the value which is closer to zero: -0.5 becomes 0, 0.5 becomes 0. */
	HalfToZero,
	/** Rounds to the value which is further from zero, "larger" in absolute value: 0.1 becomes 1, -0.1 becomes -1 */
	FromZero,
	/** Rounds to the value which is closer to zero, "smaller" in absolute value: 0.1 becomes 0, -0.1 becomes 0 */
	ToZero,
	/** Rounds to the value which is more negative: 0.1 becomes 0, -0.1 becomes -1 */
	ToNegativeInfinity,
	/** Rounds to the value which is more positive: 0.1 becomes 1, -0.1 becomes 0 */
	ToPositiveInfinity,


	// Add new enum types at the end only! They are serialized by index.
};
CORE_API bool LexTryParseString(ERoundingMode& OutValue, const TCHAR* Buffer);
CORE_API void LexFromString(ERoundingMode& OutValue, const TCHAR* Buffer);
CORE_API const TCHAR* LexToString(ERoundingMode InValue);

/** Redeclared in KismetTextLibrary for meta-data extraction purposes, be sure to update there as well */
enum EMemoryUnitStandard : int
{
	/* International Electrotechnical Commission (MiB) 1024-based */
	IEC,
	/* International System of Units 1000-based */
	SI
};

/** 
 * Specific memory unit to use when formatting memory sizes. IEC units are 1024-based (KiB, MiB, ...), SI units are 1000-based (KB, MB, ...).
 * @note Redeclared in KismetTextLibrary for meta-data extraction purposes, be sure to update there as well.
 */
enum class EMemoryUnit : uint8
{
	Unit_Bytes,

	/* Auto-select based on magnitude using IEC */
	Auto_IEC,

	/* IEC (1024-based) */
	Unit_KiB, Unit_MiB, Unit_GiB, Unit_TiB, Unit_PiB, Unit_EiB,

	/* Auto-select based on magnitude using SI */
	Auto_SI,

	/* SI (1000-based) */
	Unit_KB, Unit_MB, Unit_GB, Unit_TB, Unit_PB, Unit_EB
};

struct FNumberFormattingOptions
{
	CORE_API FNumberFormattingOptions();

	bool AlwaysSign;
	FNumberFormattingOptions& SetAlwaysSign( bool InValue ){ AlwaysSign = InValue; return *this; }

	bool UseGrouping;
	FNumberFormattingOptions& SetUseGrouping( bool InValue ){ UseGrouping = InValue; return *this; }

	bool IndicateNearlyInteger;
	FNumberFormattingOptions& SetIndicateNearlyInteger( bool InValue ) { IndicateNearlyInteger = InValue; return *this; }

	ERoundingMode RoundingMode;
	FNumberFormattingOptions& SetRoundingMode( ERoundingMode InValue ){ RoundingMode = InValue; return *this; }

	int32 MinimumIntegralDigits;
	FNumberFormattingOptions& SetMinimumIntegralDigits( int32 InValue ){ MinimumIntegralDigits = InValue; return *this; }

	int32 MaximumIntegralDigits;
	FNumberFormattingOptions& SetMaximumIntegralDigits( int32 InValue ){ MaximumIntegralDigits = InValue; return *this; }

	int32 MinimumFractionalDigits;
	FNumberFormattingOptions& SetMinimumFractionalDigits( int32 InValue ){ MinimumFractionalDigits = InValue; return *this; }

	int32 MaximumFractionalDigits;
	FNumberFormattingOptions& SetMaximumFractionalDigits( int32 InValue ){ MaximumFractionalDigits = InValue; return *this; }

	friend CORE_API void operator<<(FStructuredArchive::FSlot Slot, FNumberFormattingOptions& Value);

	/** Get the hash code to use for the given formatting options */
	friend CORE_API uint32 GetTypeHash( const FNumberFormattingOptions& Key );

	/** Check to see if our formatting options match the other formatting options */
	CORE_API bool IsIdentical( const FNumberFormattingOptions& Other ) const;

	/** Get the default number formatting options with grouping enabled */
	static CORE_API const FNumberFormattingOptions& DefaultWithGrouping();

	/** Get the default number formatting options with grouping disabled */
	static CORE_API const FNumberFormattingOptions& DefaultNoGrouping();
};

struct FNumberParsingOptions
{
	CORE_API FNumberParsingOptions();

	bool UseGrouping;
	FNumberParsingOptions& SetUseGrouping( bool InValue ){ UseGrouping = InValue; return *this; }

	/** The number needs to be representable inside its type limits to be considered valid. */
	bool InsideLimits;
	FNumberParsingOptions& SetInsideLimits(bool InValue) { InsideLimits = InValue; return *this; }

	/** Clamp the parsed value to its type limits. */
	bool UseClamping;
	FNumberParsingOptions& SetUseClamping(bool InValue) { UseClamping = InValue; return *this; }

	friend CORE_API void operator<<(FStructuredArchive::FSlot Slot, FNumberParsingOptions& Value);

	/** Get the hash code to use for the given parsing options */
	friend CORE_API uint32 GetTypeHash( const FNumberParsingOptions& Key );

	/** Check to see if our parsing options match the other parsing options */
	CORE_API bool IsIdentical( const FNumberParsingOptions& Other ) const;

	/** Get the default number parsing options with grouping enabled */
	static CORE_API const FNumberParsingOptions& DefaultWithGrouping();

	/** Get the default number parsing options with grouping disabled */
	static CORE_API const FNumberParsingOptions& DefaultNoGrouping();
};

/**
 * Cached compiled expression used by the text formatter.
 * The compiled expression will automatically update if the display string is changed,
 * and is safe to be used as a function-level static.
 * See TextFormatter.cpp for the definition.
 */
class FTextFormat
{
	friend class FTextFormatter;

public:
	enum class EExpressionType
	{
		/** Invalid expression */
		Invalid,
		/** Simple expression, containing no arguments or argument modifiers */
		Simple,
		/** Complex expression, containing arguments or argument modifiers */
		Complex,
	};

	/**
	 * Construct an instance using an empty FText.
	 */
	CORE_API FTextFormat();

	/**
	 * Construct an instance from an FText.
	 * The text will be immediately compiled. 
	 */
	CORE_API FTextFormat(const FText& InText, ETextFormatFlags InFormatFlags = ETextFormatFlags::Default);

	/**
	 * Construct an instance from an FText and custom format pattern definition.
	 * The text will be immediately compiled.
	 */
	CORE_API FTextFormat(const FText& InText, FTextFormatPatternDefinitionConstRef InCustomPatternDef, ETextFormatFlags InFormatFlags = ETextFormatFlags::Default);

	/**
	 * Construct an instance from an FString.
	 * The string will be immediately compiled.
	 */
	static CORE_API FTextFormat FromString(const FString& InString, ETextFormatFlags InFormatFlags = ETextFormatFlags::Default);
	static CORE_API FTextFormat FromString(FString&& InString, ETextFormatFlags InFormatFlags = ETextFormatFlags::Default);

	/**
	 * Construct an instance from an FString and custom format pattern definition.
	 * The string will be immediately compiled.
	 */
	static CORE_API FTextFormat FromString(const FString& InString, FTextFormatPatternDefinitionConstRef InCustomPatternDef, ETextFormatFlags InFormatFlags = ETextFormatFlags::Default);
	static CORE_API FTextFormat FromString(FString&& InString, FTextFormatPatternDefinitionConstRef InCustomPatternDef, ETextFormatFlags InFormatFlags = ETextFormatFlags::Default);

	/**
	 * Test to see whether this instance contains valid compiled data.
	 */
	CORE_API bool IsValid() const;

	/**
	 * Check whether this instance is considered identical to the other instance, based on the comparison flags provided.
	 */
	CORE_API bool IdenticalTo(const FTextFormat& Other, const ETextIdenticalModeFlags CompareModeFlags) const;

	/**
	 * Get the source text that we're holding.
	 * If we're holding a string then we'll construct a new text.
	 */
	CORE_API FText GetSourceText() const;

	/**
	 * Get the source string that we're holding.
	 * If we're holding a text then we'll return its internal string.
	 */
	CORE_API const FString& GetSourceString() const;

	/**
	 * Get the type of expression currently compiled.
	 */
	CORE_API EExpressionType GetExpressionType() const;

	/**
	 * Get the format flags being used.
	 */
	CORE_API ETextFormatFlags GetFormatFlags() const;

	/**
	 * Get the format pattern definition being used.
	 */
	CORE_API FTextFormatPatternDefinitionConstRef GetPatternDefinition() const;

	/**
	 * Validate the format pattern is valid based on the rules of the given culture (or null to use the current language).
	 * @return true if the pattern is valid, or false if not (false may also fill in OutValidationErrors).
	 */
	CORE_API bool ValidatePattern(const FCulturePtr& InCulture, TArray<FString>& OutValidationErrors) const;

	/**
	 * Append the names of any arguments to the given array.
	 */
	CORE_API void GetFormatArgumentNames(TArray<FString>& OutArgumentNames) const;

private:
	/**
	 * Construct an instance from an FString.
	 * The string will be immediately compiled.
	 */
	CORE_API FTextFormat(FString&& InString, FTextFormatPatternDefinitionConstRef InCustomPatternDef, ETextFormatFlags InFormatFlags);

	/** Cached compiled expression data */
	TSharedRef<FTextFormatData, ESPMode::ThreadSafe> TextFormatData;
};

class FCulture;

class FText
{
public:

	static CORE_API const FText& GetEmpty();

public:

	CORE_API FText();
	
	FText(const FText&) = default;
	FText& operator=(const FText&) = default;
	
	CORE_API FText(FText&& Other);
	CORE_API FText& operator=(FText&& Other);

	// FText can be returned from AutoRTFM::Open() via a copy-construction in the closed.
	static constexpr AutoRTFM::EReturnFromOpenMode AutoRTFMReturnFromOpenMode = AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed;

	/**
	 * Generate an FText that represents the passed number in the current culture
	 */
	[[nodiscard]] static CORE_API FText AsNumber(float Val,	const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsNumber(double Val,	const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsNumber(int8 Val,		const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsNumber(int16 Val,	const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsNumber(int32 Val,	const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsNumber(int64 Val,	const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsNumber(uint8 Val,	const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsNumber(uint16 Val,	const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsNumber(uint32 Val,	const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsNumber(uint64 Val,	const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsNumber(long Val,		const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);


	[[nodiscard]] static CORE_API FText AsCurrency(float Val,  const FString& CurrencyCode = FString(), const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsCurrency(double Val, const FString& CurrencyCode = FString(), const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsCurrency(int8 Val,   const FString& CurrencyCode = FString(), const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsCurrency(int16 Val,  const FString& CurrencyCode = FString(), const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsCurrency(int32 Val,  const FString& CurrencyCode = FString(), const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsCurrency(int64 Val,  const FString& CurrencyCode = FString(), const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsCurrency(uint8 Val,  const FString& CurrencyCode = FString(), const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsCurrency(uint16 Val, const FString& CurrencyCode = FString(), const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsCurrency(uint32 Val, const FString& CurrencyCode = FString(), const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsCurrency(uint64 Val, const FString& CurrencyCode = FString(), const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsCurrency(long Val,   const FString& CurrencyCode = FString(), const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);

	/**
	 * Generate an FText that represents the passed number as currency in the current culture.
	 * BaseVal is specified in the smallest fractional value of the currency and will be converted for formatting according to the selected culture.
	 * Keep in mind the CurrencyCode is completely independent of the culture it's displayed in (and they do not imply one another).
	 * For example: FText::AsCurrencyBase(650, TEXT("EUR")); would return an FText of "<EUR>6.50" in most English cultures (en_US/en_UK) and "6,50<EUR>" in Spanish (es_ES) (where <EUR> is U+20AC)
	 */
	[[nodiscard]] static CORE_API FText AsCurrencyBase(int64 BaseVal, const FString& CurrencyCode, const FCulturePtr& TargetCulture = NULL, int32 ForceDecimalPlaces = -1);

	/**
	 * Generate an FText that represents the passed number as a percentage in the current culture
	 */
	[[nodiscard]] static CORE_API FText AsPercent(float Val,	const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsPercent(double Val,	const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL);

	/**
	 * Generate an FText that represents the given timestamp as a date/time in the current culture using the given timezone (default is the local timezone).
	 * 
	 * This function will convert the given timestamp from UTC (eg, FDateTime::UtcNow()) to the given timezone (taking into account DST).
	 * If using a local timestamp (eg, FDateTime::Now()) then you MUST use FText::GetInvariantTimeZone() as the TimeZone argument to avoid additional timezone conversion.
	 * 
	 * @note The overload using a custom pattern uses strftime-like syntax (see FDateTime::ToFormattedString).
	 */
	[[nodiscard]] static CORE_API FText AsDate(const FDateTime& DateTime, const EDateTimeStyle::Type DateStyle = EDateTimeStyle::Default, const FString& TimeZone = FString(), const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsDateTime(const FDateTime& DateTime, const EDateTimeStyle::Type DateStyle = EDateTimeStyle::Default, const EDateTimeStyle::Type TimeStyle = EDateTimeStyle::Default, const FString& TimeZone = FString(), const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsDateTime(const FDateTime& DateTime, const FString& CustomPattern, const FString& TimeZone = FString(), const FCulturePtr& TargetCulture = NULL);
	[[nodiscard]] static CORE_API FText AsTime(const FDateTime& DateTime, const EDateTimeStyle::Type TimeStyle = EDateTimeStyle::Default, const FString& TimeZone = FString(), const FCulturePtr& TargetCulture = NULL);

	/**
	 * Generate an FText that represents the given timespan in the current culture, as a delta-time representation (eg, {Hours}:{Minutes}:{Seconds} or {Minutes}:{Seconds}).
	 */
	[[nodiscard]] static CORE_API FText AsTimespan(const FTimespan& Timespan, const FCulturePtr& TargetCulture = NULL);

	/**
	 * Gets the time zone string that represents a non-specific, zero offset, culture invariant time zone.
	 */
	[[nodiscard]] static CORE_API FString GetInvariantTimeZone();

	/**
	 * Generate an FText that represents the passed number as a memory size in the current culture
	 */
	[[nodiscard]] static CORE_API FText AsMemory(uint64 NumBytes, const FNumberFormattingOptions* const Options = NULL, const FCulturePtr& TargetCulture = NULL, EMemoryUnitStandard UnitStandard = EMemoryUnitStandard::IEC);

	/**
	 * Generate an FText that represents the passed number as a memory size in the current culture
	 */
	[[nodiscard]] static CORE_API FText AsMemory(uint64 NumBytes, EMemoryUnitStandard UnitStandard);

	/**
	 * Generate an FText that represents the passed number as a memory size in the current culture, expressed in a specific unit.
	 * Use EMemoryUnit::Auto_IEC or EMemoryUnit::Auto_SI to preserve the default magnitude-based selection behavior.
	 */
	[[nodiscard]] static CORE_API FText AsMemory(uint64 NumBytes, EMemoryUnit Unit, const FNumberFormattingOptions* const Options = nullptr, const FCulturePtr& TargetCulture = nullptr);

	/**
	 * Attempts to create an FText instance from a string table ID and key (this is the same as the LOCTABLE macro, except this can also work with non-literal string values).
	 * @return The found text, or a dummy FText if not found.
	 */
	[[nodiscard]] static CORE_API FText FromStringTable(const FName InTableId, const FTextKey& InKey, const EStringTableLoadingPolicy InLoadingPolicy = EStringTableLoadingPolicy::FindOrLoad);

	/**
	 * Generate an FText representing the pass name
	 */
	[[nodiscard]] static CORE_API FText FromName( const FName& Val);
	
	/**
	 * Generate an FText representing the passed in string
	 */
	[[nodiscard]] static CORE_API FText FromString( const ANSICHAR* String );
	[[nodiscard]] static CORE_API FText FromString( const UTF8CHAR* String );
	[[nodiscard]] static CORE_API FText FromString( const WIDECHAR* String );
	[[nodiscard]] static CORE_API FText FromString( const FAnsiString& String );
	[[nodiscard]] static CORE_API FText FromString( const FString& String );
	[[nodiscard]] static CORE_API FText FromString( FString&& String );
	[[nodiscard]] static CORE_API FText FromString( const FUtf8String& String );

	/**
	 * Generate a FText representing the passed string view
	 */
	[[nodiscard]] static CORE_API FText FromStringView(FAnsiStringView InString);
	[[nodiscard]] static CORE_API FText FromStringView(FUtf8StringView InString);
	[[nodiscard]] static CORE_API FText FromStringView(FWideStringView InString);

	/**
	 * Generate a culture invariant FText representing the passed in string
	 */
	[[nodiscard]] static CORE_API FText AsCultureInvariant( const ANSICHAR* String );
	[[nodiscard]] static CORE_API FText AsCultureInvariant( const UTF8CHAR* String );
	[[nodiscard]] static CORE_API FText AsCultureInvariant( const WIDECHAR* String );
	[[nodiscard]] static CORE_API FText AsCultureInvariant( FAnsiStringView String );
	[[nodiscard]] static CORE_API FText AsCultureInvariant( FUtf8StringView String );
	[[nodiscard]] static CORE_API FText AsCultureInvariant( FWideStringView String );
	[[nodiscard]] static CORE_API FText AsCultureInvariant( FString&& String );

	/**
	 * Generate a culture invariant FText representing the passed in FText
	 */
	[[nodiscard]] static CORE_API FText AsCultureInvariant( FText Text );

	/**
	 * === !! This is an ADVANCED function. USE WITH CAUTION !! ===
	 * 
	 * Generate a localizable FText for the given ID and source string, either to create a NEW localizable FText, or to dynamically reference an EXISTING FText.
	 * 
	 * When using it to create a NEW localizable FText (eg, when editing an asset), YOU are responsible for ensuring that the resultant FText is gathered for localization (eg, by assigning it to a FText property or via a CUSTOM gather handler).
	 * Note: DO NOT use this as a general replacement for the LOCTEXT macro in C++!
	 * 
	 * When using it to dynamically reference an EXISTING FText (such as known LOCTEXT in code) it can be thought of as an alternative to FText::FindTextInLiveTable_Advanced that also handles untranslated text.
	 * Note: Direct dynamic references to FText are EXTREMELY FRAGILE, and you may want to use a string table instead!
	 */
#if WITH_EDITORONLY_DATA
	[[nodiscard]] static CORE_API FText AsLocalizable_Advanced(const FTextKey& Namespace, const FTextKey& Key, const TCHAR* String, FStringView InDevNotes = TEXTVIEW(""));
	[[nodiscard]] static CORE_API FText AsLocalizable_Advanced(const FTextKey& Namespace, const FTextKey& Key, FStringView String, FStringView InDevNotes = TEXTVIEW(""));
	[[nodiscard]] static CORE_API FText AsLocalizable_Advanced(const FTextKey& Namespace, const FTextKey& Key, FString&& String, FStringView InDevNotes = TEXTVIEW(""));
	[[nodiscard]] static CORE_API FText AsLocalizable_Advanced_LocText(const TCHAR* Namespace, const TCHAR* Key, const TCHAR* String, FStringView InDevNotes = TEXTVIEW("")); // For use by LOCTEXT to avoid constructing temp FTextKeys in every caller
#else
	[[nodiscard]] static CORE_API FText AsLocalizable_Advanced(const FTextKey& Namespace, const FTextKey& Key, const TCHAR* String);
	[[nodiscard]] static CORE_API FText AsLocalizable_Advanced(const FTextKey& Namespace, const FTextKey& Key, FStringView String);
	[[nodiscard]] static CORE_API FText AsLocalizable_Advanced(const FTextKey& Namespace, const FTextKey& Key, FString&& String);
	[[nodiscard]] static CORE_API FText AsLocalizable_Advanced_LocText(const TCHAR* Namespace, const TCHAR* Key, const TCHAR* String); // For use by LOCTEXT to avoid constructing temp FTextKeys in every caller
#endif

	/**
	 * === !! This is an ADVANCED function. USE WITH CAUTION !! ===
	 *
	 * Attempt to dynamically reference an EXISTING FText via its active display string in the live table.
	 * Note: This can ONLY find text that is currently localized (gathered, translated, and has an active display string in FTextLocalizationManager). If you need to find a localizable but untranslated text, see FText::AsLocalizable_Advanced.
	 * Note: Direct dynamic references to FText are EXTREMELY FRAGILE, and you may want to use a string table instead!
	 * 
	 * @param Namespace The namespace of the text to find (if any).
	 * @param Key The key of the text to find.
	 * @param SourceString If set (not empty) then the found text must also have been created from this source string.
	 * 
	 * @return true if OutText was set; otherwise false and OutText will be unmodified
	 */
	static CORE_API bool FindTextInLiveTable_Advanced(const FTextKey& Namespace, const FTextKey& Key, FText& OutText, const FString* const SourceString = nullptr);

	[[nodiscard]] CORE_API const FString& ToString() const;

	/** Deep build of the source string for this FText, climbing the history hierarchy */
	[[nodiscard]] CORE_API FString BuildSourceString() const;

	[[nodiscard]] CORE_API bool IsNumeric() const;

	[[nodiscard]] CORE_API int32 CompareTo( const FText& Other, const ETextComparisonLevel::Type ComparisonLevel = ETextComparisonLevel::Default ) const;
	[[nodiscard]] CORE_API int32 CompareToCaseIgnored( const FText& Other ) const;

	[[nodiscard]] CORE_API bool EqualTo( const FText& Other, const ETextComparisonLevel::Type ComparisonLevel = ETextComparisonLevel::Default ) const;
	[[nodiscard]] CORE_API bool EqualToCaseIgnored( const FText& Other ) const;

	/**
	 * Check to see if this FText is identical to the other FText
	 * 
	 * @note This function defaults to only testing that the internal data has the same target (which makes it very fast!), rather than performing any deep or lexical analysis.
	 *       The ETextIdenticalModeFlags can modify this default behavior. See the comments on those flag options for more information.
	 *
	 * @note If you actually want to perform a full lexical comparison, then you need to use EqualTo instead.
	 */
	[[nodiscard]] CORE_API bool IdenticalTo( const FText& Other, const ETextIdenticalModeFlags CompareModeFlags = ETextIdenticalModeFlags::None ) const;

	class FSortPredicate
	{
	public:
		CORE_API FSortPredicate(const ETextComparisonLevel::Type ComparisonLevel = ETextComparisonLevel::Default);

		CORE_API bool operator()(const FText& A, const FText& B) const;

	private:
#if UE_ENABLE_ICU
		class FSortPredicateImplementation;
		TSharedRef<FSortPredicateImplementation> Implementation;
#endif
	};

	[[nodiscard]] CORE_API bool IsEmpty() const;

	[[nodiscard]] CORE_API bool IsEmptyOrWhitespace() const;

	/**
	 * Transforms the text to lowercase in a culture correct way.
	 * @note The returned instance is linked to the original and will be rebuilt if the active culture is changed.
	 */
	[[nodiscard]] CORE_API FText ToLower() const;

	/**
	 * Transforms the text to uppercase in a culture correct way.
	 * @note The returned instance is linked to the original and will be rebuilt if the active culture is changed.
	 */
	[[nodiscard]] CORE_API FText ToUpper() const;

	/**
	 * Removes any whitespace characters from the start of the text.
	 */
	[[nodiscard]] static CORE_API FText TrimPreceding( const FText& );

	/**
	 * Removes any whitespace characters from the end of the text.
	 */
	[[nodiscard]] static CORE_API FText TrimTrailing( const FText& );

	/**
	 * Removes any whitespace characters from the start and end of the text.
	 */
	[[nodiscard]] static CORE_API FText TrimPrecedingAndTrailing( const FText& );

	/**
	 * Check to see if the given character is considered whitespace by the current culture
	 */
	UE_DEPRECATED(5.7, "FText::IsWhitespace is deprecated. Use FTextChar::IsWhitespace instead.")
	[[nodiscard]] static CORE_API bool IsWhitespace( const TCHAR Char );

	static CORE_API void GetFormatPatternParameters(const FTextFormat& Fmt, TArray<FString>& ParameterNames);

	/**
	 * Format the given map of key->value pairs as named arguments within the given format pattern
	 *
	 * @note You may want to pre-compile your FText pattern into a FTextFormat prior to performing formats within a loop or on a critical path,
	 *       as this can save CPU cycles, memory, and mutex resources vs re-compiling the pattern for each format call. See FTextFormat for more info.
	 *
	 * @param Fmt The format pattern to use
	 * @param InArguments The map of key->value pairs to inject into the format pattern
	 * @return The formatted FText
	 */
	[[nodiscard]] static CORE_API FText Format(FTextFormat Fmt, const FFormatNamedArguments& InArguments);
	[[nodiscard]] static CORE_API FText Format(FTextFormat Fmt, FFormatNamedArguments&& InArguments);

	/**
	 * Format the given list values as ordered arguments within the given format pattern
	 *
	 * @note You may want to pre-compile your FText pattern into a FTextFormat prior to performing formats within a loop or on a critical path,
	 *       as this can save CPU cycles, memory, and mutex resources vs re-compiling the pattern for each format call. See FTextFormat for more info.
	 *
	 * @param Fmt The format pattern to use
	 * @param InArguments The list of values to inject into the format pattern
	 * @return The formatted FText
	 */
	[[nodiscard]] static CORE_API FText Format(FTextFormat Fmt, const FFormatOrderedArguments& InArguments);
	[[nodiscard]] static CORE_API FText Format(FTextFormat Fmt, FFormatOrderedArguments&& InArguments);

	/**
	 * Format the given list of variadic values as ordered arguments within the given format pattern
	 *
	 * @note You may want to pre-compile your FText pattern into a FTextFormat prior to performing formats within a loop or on a critical path, 
	 *       as this can save CPU cycles, memory, and mutex resources vs re-compiling the pattern for each format call. See FTextFormat for more info.
	 * 
	 * @usage FText::Format(LOCTEXT("PlayerNameFmt", "{0} is really cool"), FText::FromString(PlayerName));
	 * 
	 * @param Fmt The format pattern to use
	 * @param Args A variadic list of values to inject into the format pattern
	 * @return The formatted FText
	 */
	template <typename... ArgTypes>
	[[nodiscard]] static inline FText Format(FTextFormat Fmt, ArgTypes... Args)
	{
		static_assert((TIsConstructible<FFormatArgumentValue, ArgTypes>::Value && ...), "Invalid argument type passed to FText::Format");
		static_assert(sizeof...(Args) > 0, "FText::Format expects at least one non-format argument"); // we do this to ensure that people don't call Format for no good reason

		// We do this to force-select the correct overload, because overload resolution will cause compile
		// errors when it tries to instantiate the FFormatNamedArguments overloads.
		FText (*CorrectFormat)(FTextFormat, FFormatOrderedArguments&&) = Format;

		return CorrectFormat(MoveTemp(Fmt), FFormatOrderedArguments{ MoveTemp(Args)... });
	}

	/**
	 * Format the given list of variadic key->value pairs as named arguments within the given format pattern
	 *
	 * @note You may want to pre-compile your FText pattern into a FTextFormat prior to performing formats within a loop or on a critical path,
	 *       as this can save CPU cycles, memory, and mutex resources vs re-compiling the pattern for each format call. See FTextFormat for more info.
	 * 
	 * @usage FText::FormatNamed(LOCTEXT("PlayerNameFmt", "{PlayerName} is really cool"), TEXT("PlayerName"), FText::FromString(PlayerName));
	 * 
	 * @param Fmt The format pattern to use
	 * @param Args A variadic list of "key then value" pairs to inject into the format pattern (must be an even number)
	 * @return The formatted FText
	 */
	template < typename... TArguments >
	[[nodiscard]] static FText FormatNamed( FTextFormat Fmt, TArguments&&... Args );

	/**
	 * Format the given list of variadic values as ordered arguments within the given format pattern
	 *
	 * @note You may want to pre-compile your FText pattern into a FTextFormat prior to performing formats within a loop or on a critical path,
	 *       as this can save CPU cycles, memory, and mutex resources vs re-compiling the pattern for each format call. See FTextFormat for more info.
	 * 
	 * @usage FText::FormatOrdered(LOCTEXT("PlayerNameFmt", "{0} is really cool"), FText::FromString(PlayerName));
	 * 
	 * @param Fmt The format pattern to use
	 * @param Args A variadic list of values to inject into the format pattern
	 * @return The formatted FText
	 */
	template < typename... TArguments >
	[[nodiscard]] static FText FormatOrdered( FTextFormat Fmt, TArguments&&... Args );

	/**
	 * Join an arbitrary list of formattable values together, separated by the given delimiter
	 * @note Internally this uses FText::Format with a generated culture invariant format pattern
	 *
	 * @param Delimiter The delimiter to insert between the items
	 * @param Args An array of formattable values to join together
	 * @return The joined FText
	 */
	[[nodiscard]] static CORE_API FText Join(const FText& Delimiter, const FFormatOrderedArguments& Args);
	[[nodiscard]] static CORE_API FText Join(const FText& Delimiter, const TArray<FText>& Args);

	/**
	 * Join an arbitrary list of formattable items together, separated by the given delimiter
	 * @note Internally this uses FText::Format with a generated culture invariant format pattern
	 *
	 * @param Delimiter The delimiter to insert between the items
	 * @param Args A variadic list of values to join together
	 * @return The joined FText
	 */
	template <typename... ArgTypes>
	[[nodiscard]] static inline FText Join(const FText& Delimiter, ArgTypes... Args)
	{
		static_assert((TIsConstructible<FFormatArgumentValue, ArgTypes>::Value && ...), "Invalid argument type passed to FText::Join");
		static_assert(sizeof...(Args) > 0, "FText::Join expects at least one non-format argument"); // we do this to ensure that people don't call Join for no good reason

		return Join(Delimiter, FFormatOrderedArguments{ MoveTemp(Args)... });
	}

	/**
	 * Produces a custom-generated FText. Can be used for objects that produce text dependent on localized strings but
	 * that do not fit the standard formats.
	 *
	 * @param TextGenerator the text generator object that will generate the text
	 */
	[[nodiscard]] static CORE_API FText FromTextGenerator( const TSharedRef<ITextGenerator>& TextGenerator );

	DECLARE_DELEGATE_RetVal_OneParam( TSharedRef<ITextGenerator>, FCreateTextGeneratorDelegate, FStructuredArchive::FRecord );
	/**
	 * Returns the text generator factory function registered under the specified name, if any.
	 *
	 * @param TypeID the name under which to look up the factory function
	 */
	[[nodiscard]] static CORE_API FCreateTextGeneratorDelegate FindRegisteredTextGenerator( FName TypeID );

	/**
	 * Registers a factory function to be used with serialization of text generators within FText.
	 *
	 * @param TypeID the name under which to register the factory function. Must match ITextGenerator::GetTypeID().
	 * @param FactoryFunction the factory function to create the generator instance
	 */
	static CORE_API void RegisterTextGenerator( FName TypeID, FCreateTextGeneratorDelegate FactoryFunction );

	/**
	 * Registers a standard text generator factory function.
	 *
	 * @tparam T the text generator class type
	 *
	 * @param TypeID the name under which to register the factor function
	 */
	template < typename T >
	static void RegisterTextGenerator( FName TypeID )
	{
		RegisterTextGenerator(TypeID, FCreateTextGeneratorDelegate::CreateStatic( &CreateTextGenerator<T> ));
	}

	/**
	 * Registers a standard text generator factory function.
	 *
	 * @tparam T the text generator class type
	 *
	 * This function can be used if the class has a public static FName member named "TypeID".
	 */
	template < typename T >
	static void RegisterTextGenerator()
	{
		RegisterTextGenerator<T>( T::TypeID );
	}

	/**
	 * Unregisters a factory function to be used with serialization of text generators within FText.
	 *
	 * @param TypeID the name to remove from registration
	 *
	 * @see RegisterTextGenerator
	 */
	static CORE_API void UnregisterTextGenerator( FName TypeID );

	/**
	 * Unregisters a standard text generator factory function.
	 *
	 * This function can be used if the class has a public static FName member named "TypeID".
	 *
	 * @tparam T the text generator class type
	 */
	template < typename T >
	static void UnregisterTextGenerator()
	{
		UnregisterTextGenerator( T::TypeID );
	}

	[[nodiscard]] CORE_API bool IsTransient() const;
	[[nodiscard]] CORE_API bool IsCultureInvariant() const;
	[[nodiscard]] CORE_API bool IsInitializedFromString() const;
	[[nodiscard]] CORE_API bool IsFromStringTable() const;

	[[nodiscard]] CORE_API bool ShouldGatherForLocalization() const;
	[[nodiscard]] static CORE_API bool ShouldGatherForLocalization(FStringView SourceString);
	
	static CORE_API void PreloadStringTable( FName InTableId, const FTextKey& InKey );

#if WITH_EDITORONLY_DATA
	/**
	 * Constructs a new FText with the SourceString of the specified text but with the specified namespace and key
	 */
	[[nodiscard]] static CORE_API FText ChangeKey( const FTextKey& Namespace, const FTextKey& Key, const FText& Text );

	/**
	 * Constructs a new FText with the SourceString of the specified text but with the specified namespace, key, and developer translation context notes
	 */
	[[nodiscard]] static CORE_API FText ChangeKey( const FTextKey& Namespace, const FTextKey& Key, const FText& Text, const FString& DevNotes );

	CORE_API const FString& GetDevNotes() const;
#endif

	/////////////////////////////////////////////////////
	// Start - intrusive TOptional<FText> state        //
	/////////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = FText;

	explicit FText(FIntrusiveUnsetOptionalState)
	: TextData(nullptr)
	, Flags(0)
	{
	}
	bool operator==(FIntrusiveUnsetOptionalState) const
	{
		return TextData == nullptr;
	}
	/////////////////////////////////////////////////
	// End - intrusive TOptional<FText> state      //
	/////////////////////////////////////////////////
private:
	template <
		typename HistoryType
		UE_REQUIRES(std::is_convertible_v<HistoryType*, ITextData*>)
	>
	explicit FText( TRefCountPtr<HistoryType>&& InTextData )
		: TextData(MoveTemp(InTextData))
		, Flags(0)
	{
	}

	CORE_API explicit FText( FString&& InSourceString );

	CORE_API FText( FName InTableId, const FTextKey& InKey, const EStringTableLoadingPolicy InLoadingPolicy );

	CORE_API FText( FString&& InSourceString, const FTextKey& InNamespace, const FTextKey& InKey, uint32 InFlags=0 );
	
#if WITH_EDITORONLY_DATA
	CORE_API FText( FString&& InSourceString, const FTextKey& InNamespace, const FTextKey& InKey, FString InDevNotes, uint32 InFlags=0 );
#endif

	static CORE_API void SerializeText( FArchive& Ar, FText& Value );
	static CORE_API void SerializeText(FStructuredArchive::FSlot Slot, FText& Value);

	/** Get any historic text format data from the history used by this FText */
	CORE_API void GetHistoricFormatData(TArray<FHistoricTextFormatData>& OutHistoricFormatData) const;

	/** Get any historic numeric format data from the history used by this FText */
	CORE_API bool GetHistoricNumericData(FHistoricTextNumericData& OutHistoricNumericData) const;

	/** Rebuilds the FText under the current culture if needed */
	CORE_API void Rebuild() const;

	static CORE_API FText FormatNamedImpl(FTextFormat&& Fmt, FFormatNamedArguments&& InArguments);
	static CORE_API FText FormatOrderedImpl(FTextFormat&& Fmt, FFormatOrderedArguments&& InArguments);

private:
	template<typename T1, typename T2>
	static FText AsNumberTemplate(T1 Val, const FNumberFormattingOptions* const Options, const FCulturePtr& TargetCulture);
	template<typename T1, typename T2>
	static FText AsCurrencyTemplate(T1 Val, const FString& CurrencyCode, const FNumberFormattingOptions* const Options, const FCulturePtr& TargetCulture);
	template<typename T1, typename T2>
	static FText AsPercentTemplate(T1 Val, const FNumberFormattingOptions* const Options, const FCulturePtr& TargetCulture);

private:
	template < typename T >
	static TSharedRef<ITextGenerator> CreateTextGenerator(FStructuredArchive::FRecord Record);

	/** This accessor for TextData guards against the case where TextData might be null and returns the ITextData from EmptyText instead. */
	const ITextData& GetTextData() const;

private:
	/** The internal shared data for this FText */
	TRefCountPtr<ITextData> TextData;

	/** Flags with various information on what sort of FText this is */
	uint32 Flags;

public:
	friend class FTextCache;
	friend class FTextFormatter;
	friend class FTextFormatData;
	friend class FTextSnapshot;
	friend class FTextInspector;
	friend class FTextStringHelper;
	friend class FStringTableRegistry;
	friend class FArchive;
	friend class FArchiveFromStructuredArchiveImpl;
	friend class FJsonArchiveInputFormatter;
	friend class FJsonArchiveOutputFormatter;
	friend class FTextProperty;
	friend class FFormatArgumentValue;
	friend class FTextHistory_NamedFormat;
	friend class FTextHistory_ArgumentDataFormat;
	friend class FTextHistory_OrderedFormat;
	friend class FTextHistory_Transform;
	friend class FScopedTextIdentityPreserver;
};

class FFormatArgumentValue
{
public:
	FFormatArgumentValue()
		: Type(EFormatArgumentType::Text)
		, TextValue(FText::GetEmpty())
	{
	}

	CORE_API FFormatArgumentValue(const class FCbValue& Value);

	FFormatArgumentValue(const int32 Value)
		: Type(EFormatArgumentType::Int)
	{
		IntValue = Value;
	}

	FFormatArgumentValue(const uint32 Value)
		: Type(EFormatArgumentType::UInt)
	{
		UIntValue = Value;
	}

	FFormatArgumentValue(const int64 Value)
		: Type(EFormatArgumentType::Int)
	{
		IntValue = Value;
	}

	FFormatArgumentValue(const uint64 Value)
		: Type(EFormatArgumentType::UInt)
	{
		UIntValue = Value;
	}

	FFormatArgumentValue(const float Value)
		: Type(EFormatArgumentType::Float)
	{
		FloatValue = Value;
	}

	FFormatArgumentValue(const double Value)
		: Type(EFormatArgumentType::Double)
	{
		DoubleValue = Value;
	}

	FFormatArgumentValue(const FText& Value)
		: Type(EFormatArgumentType::Text)
		, TextValue(Value)
	{
	}

	FFormatArgumentValue(FText&& Value)
		: Type(EFormatArgumentType::Text)
		, TextValue(MoveTemp(Value))
	{
	}

	FFormatArgumentValue(ETextGender Value)
		: Type(EFormatArgumentType::Gender)
	{
		UIntValue = (uint64)Value;
	}

	friend void operator<<(FStructuredArchive::FSlot Slot, FFormatArgumentValue& Value);

	CORE_API bool IdenticalTo(const FFormatArgumentValue& Other, const ETextIdenticalModeFlags CompareModeFlags) const;

	CORE_API FString ToFormattedString(const bool bInRebuildText, const bool bInRebuildAsSource) const;
	CORE_API void ToFormattedString(const bool bInRebuildText, const bool bInRebuildAsSource, FString& OutResult) const;

	CORE_API FString ToExportedString(const bool bStripPackageNamespace = false) const;
	CORE_API void ToExportedString(FString& OutResult, const bool bStripPackageNamespace = false) const;
	CORE_API const TCHAR* FromExportedString(const TCHAR* InBuffer);

	UE_FORCEINLINE_HINT EFormatArgumentType::Type GetType() const
	{
		return Type;
	}

	inline int64 GetIntValue() const
	{
		check(Type == EFormatArgumentType::Int);
		return IntValue;
	}

	inline uint64 GetUIntValue() const
	{
		check(Type == EFormatArgumentType::UInt);
		return UIntValue;
	}

	inline float GetFloatValue() const
	{
		check(Type == EFormatArgumentType::Float);
		return FloatValue;
	}

	inline double GetDoubleValue() const
	{
		check(Type == EFormatArgumentType::Double);
		return DoubleValue;
	}

	inline const FText& GetTextValue() const
	{
		check(Type == EFormatArgumentType::Text);
		return TextValue.GetValue();
	}

	inline ETextGender GetGenderValue() const
	{
		check(Type == EFormatArgumentType::Gender);
		return (ETextGender)UIntValue;
	}

private:
	EFormatArgumentType::Type Type;
	union
	{
		int64 IntValue;
		uint64 UIntValue;
		float FloatValue;
		double DoubleValue;
	};
	TOptional<FText> TextValue;
};

template < typename T >
inline TSharedRef<ITextGenerator> FText::CreateTextGenerator(FStructuredArchive::FRecord Record)
{
	return MakeShared<T>();
}

/**
 * Used to pass argument/value pairs into FText::Format via UKismetTextLibrary::Format.
 * @note The primary consumer of this type is Blueprints (via a UHT mirror node). It is *not* expected that this be used in general C++ as FFormatArgumentValue is a much better type.
 * The UHT struct is located here: Engine\Source\Runtime\Engine\Classes\Kismet\KismetTextLibrary.h
 */
struct FFormatArgumentData
{
	FFormatArgumentData()
	{
		ResetValue();
	}

	CORE_API void ResetValue();

	CORE_API FFormatArgumentValue ToArgumentValue() const;

	friend void operator<<(FStructuredArchive::FSlot Slot, FFormatArgumentData& Value);

	FString ArgumentName;

	// This is a non-unioned version of FFormatArgumentValue that only accepts the types needed by Blueprints
	// It's used as a marshaller to create a real FFormatArgumentValue when performing a format
	TEnumAsByte<EFormatArgumentType::Type> ArgumentValueType;
	FText ArgumentValue;
	int64 ArgumentValueInt;
	float ArgumentValueFloat;
	double ArgumentValueDouble;
	ETextGender ArgumentValueGender;
};

namespace TextFormatUtil
{

	template < typename TName, typename TValue >
	void FormatNamed( OUT FFormatNamedArguments& Result, TName&& Name, TValue&& Value )
	{
		Result.Emplace( Forward< TName >( Name ), Forward< TValue >( Value ) );
	}
	
	template < typename TName, typename TValue, typename... TArguments >
	void FormatNamed( OUT FFormatNamedArguments& Result, TName&& Name, TValue&& Value, TArguments&&... Args )
	{
		FormatNamed( Result, Forward< TName >( Name ), Forward< TValue >( Value ) );
		FormatNamed( Result, Forward< TArguments >( Args )... );
	}
	
	template < typename TValue >
	void FormatOrdered( OUT FFormatOrderedArguments& Result, TValue&& Value )
	{
		Result.Emplace( Forward< TValue >( Value ) );
	}
	
	template < typename TValue, typename... TArguments >
	void FormatOrdered( OUT FFormatOrderedArguments& Result, TValue&& Value, TArguments&&... Args )
	{
		FormatOrdered( Result, Forward< TValue >( Value ) );
		FormatOrdered( Result, Forward< TArguments >( Args )... );
	}

} // namespace TextFormatUtil

template < typename... TArguments >
FText FText::FormatNamed( FTextFormat Fmt, TArguments&&... Args )
{
	static_assert( sizeof...( TArguments ) % 2 == 0, "FormatNamed requires an even number of Name <-> Value pairs" );

	FFormatNamedArguments FormatArguments;
	FormatArguments.Reserve( sizeof...( TArguments ) / 2 );
	TextFormatUtil::FormatNamed( FormatArguments, Forward< TArguments >( Args )... );
	return FormatNamedImpl( MoveTemp( Fmt ), MoveTemp( FormatArguments ) );
}

template < typename... TArguments >
FText FText::FormatOrdered( FTextFormat Fmt, TArguments&&... Args )
{
	FFormatOrderedArguments FormatArguments;
	FormatArguments.Reserve( sizeof...( TArguments ) );
	TextFormatUtil::FormatOrdered( FormatArguments, Forward< TArguments >( Args )... );
	return FormatOrderedImpl( MoveTemp( Fmt ), MoveTemp( FormatArguments ) );
}

/** Used to gather information about a historic text format operation */
class FHistoricTextFormatData
{
public:
	FHistoricTextFormatData()
	{
	}

	FHistoricTextFormatData(FText InFormattedText, FTextFormat&& InSourceFmt, FFormatNamedArguments&& InArguments)
		: FormattedText(MoveTemp(InFormattedText))
		, SourceFmt(MoveTemp(InSourceFmt))
		, Arguments(MoveTemp(InArguments))
	{
	}

	/** The final formatted text this data is for */
	FText FormattedText;

	/** The pattern used to format the text */
	FTextFormat SourceFmt;

	/** Arguments to replace in the pattern string */
	FFormatNamedArguments Arguments;
};

/** Used to gather information about a historic numeric format operation */
class FHistoricTextNumericData
{
public:
	enum class EType : uint8
	{
		AsNumber,
		AsPercent,
	};

	FHistoricTextNumericData()
		: FormatType(EType::AsNumber)
	{
	}

	FHistoricTextNumericData(const EType InFormatType, const FFormatArgumentValue& InSourceValue, const TOptional<FNumberFormattingOptions>& InFormatOptions)
		: FormatType(InFormatType)
		, SourceValue(InSourceValue)
		, FormatOptions(InFormatOptions)
	{
	}

	/** Type of numeric format that was performed */
	EType FormatType;

	/** The source number to format */
	FFormatArgumentValue SourceValue;

	/** Custom formatting options used when formatting this number (if any) */
	TOptional<FNumberFormattingOptions> FormatOptions;
};

/** A snapshot of an FText at a point in time that can be used to detect changes in the FText, including live-culture changes */
class FTextSnapshot
{
public:
	CORE_API FTextSnapshot();

	CORE_API explicit FTextSnapshot(const FText& InText);

	/** Check to see whether the given text is identical to the text this snapshot was made from */
	CORE_API bool IdenticalTo(const FText& InText) const;

	/** Check to see whether the display string of the given text is identical to the display string this snapshot was made from */
	CORE_API bool IsDisplayStringEqualTo(const FText& InText) const;

private:

	/** Get adjusted global history revision used for comparison */
	static uint16 GetGlobalHistoryRevisionForText(const FText& InText);

	/** Get adjusted local history revision used for comparison */
	static uint16 GetLocalHistoryRevisionForText(const FText& InText);

	/** A pointer to the text data for the FText that we took a snapshot of (used for an efficient pointer compare) */
	TRefCountPtr<ITextData> TextDataPtr;

	/** The localized string of the text when we took the snapshot (if any) */
	FTextConstDisplayStringPtr LocalizedStringPtr;

	/** Global revision index of the text when we took the snapshot, or 0 if there was no history */
	uint16 GlobalHistoryRevision = 0;

	/** Local revision index of the text when we took the snapshot, or 0 if there was no history */
	uint16 LocalHistoryRevision = 0;

	/** Flags with various information on what sort of FText we took a snapshot of */
	uint32 Flags = 0;
};

class FTextInspector
{
private:
	FTextInspector() {}
	~FTextInspector() = default;

public:
	static CORE_API bool ShouldGatherForLocalization(const FText& Text);
	static CORE_API TOptional<FString> GetNamespace(const FText& Text);
	static CORE_API TOptional<FString> GetKey(const FText& Text);
	static CORE_API FTextId GetTextId(const FText& Text);
	static CORE_API const FString* GetSourceString(const FText& Text);
	static CORE_API const FString& GetDisplayString(const FText& Text);
	static CORE_API bool GetTableIdAndKey(const FText& Text, FName& OutTableId, FString& OutKey);
	static CORE_API bool GetTableIdAndKey(const FText& Text, FName& OutTableId, FTextKey& OutKey);
	static CORE_API uint32 GetFlags(const FText& Text);
	static CORE_API void GetHistoricFormatData(const FText& Text, TArray<FHistoricTextFormatData>& OutHistoricFormatData);
	static CORE_API bool GetHistoricNumericData(const FText& Text, FHistoricTextNumericData& OutHistoricNumericData);
	static CORE_API const void* GetSharedDataId(const FText& Text);
};

class FTextStringHelper
{
public:
	/**
	 * Create an FText instance from the given stream of text.
	 * @note This uses ReadFromBuffer internally, but will fallback to FText::FromString if ReadFromBuffer fails to parse the buffer.
	 *
	 * @param Buffer			The buffer of text to read from (null terminated).
	 * @param TextNamespace		An optional namespace to use when parsing texts that use LOCTEXT (default is an empty namespace).
	 * @param PackageNamespace	The package namespace of the containing object (if loading for a property - see TextNamespaceUtil::GetPackageNamespace).
	 * @param bRequiresQuotes	True if the read text literal must be surrounded by quotes (eg, when loading from a delimited list).
	 *
	 * @return The parsed FText instance.
	 */
	static CORE_API FText CreateFromBuffer(const TCHAR* Buffer, const TCHAR* TextNamespace = nullptr, const TCHAR* PackageNamespace = nullptr, const bool bRequiresQuotes = false);

	/**
	 * Attempt to extract an FText instance from the given stream of text.
	 *
	 * @param Buffer			The buffer of text to read from (null terminated).
	 * @param OutValue			The text value to fill with the read text.
	 * @param TextNamespace		An optional namespace to use when parsing texts that use LOCTEXT (default is an empty namespace).
	 * @param PackageNamespace	The package namespace of the containing object (if loading for a property - see TextNamespaceUtil::GetPackageNamespace).
	 * @param bRequiresQuotes	True if the read text literal must be surrounded by quotes (eg, when loading from a delimited list).
	 *
	 * @return The updated buffer after we parsed this text, or nullptr on failure
	 */
	static CORE_API const TCHAR* ReadFromBuffer(const TCHAR* Buffer, FText& OutValue, const TCHAR* TextNamespace = nullptr, const TCHAR* PackageNamespace = nullptr, const bool bRequiresQuotes = false);

	/**
	 * Write the given FText instance to a stream of text
	 *
	 * @param Buffer				 The buffer of text to write to.
	 * @param Value					 The text value to write into the buffer.
	 * @param bRequiresQuotes		 True if the written text literal must be surrounded by quotes (eg, when saving as a delimited list)
	 * @param bStripPackageNamespace True to strip the package namespace from the written NSLOCTEXT value (eg, when saving cooked data)
	 */
	static CORE_API void WriteToBuffer(FString& Buffer, const FText& Value, const bool bRequiresQuotes = false, const bool bStripPackageNamespace = false);

	/**
	 * Test to see whether a given buffer contains complex text.
	 *
	 * @return True if it does, false otherwise
	 */
	static CORE_API bool IsComplexText(const TCHAR* Buffer);

private:
	static const TCHAR* ReadFromBuffer_ComplexText(const TCHAR* Buffer, FText& OutValue, const TCHAR* TextNamespace, const TCHAR* PackageNamespace);
};

class FTextBuilder
{
public:
	/**
	 * Increase the running indentation of the builder.
	 */
	CORE_API void Indent();

	/**
	 * Decrease the running indentation of the builder.
	 */
	CORE_API void Unindent();

	/**
	 * Append an empty line to the builder, indented by the running indentation of the builder.
	 */
	CORE_API void AppendLine();

	/**
	 * Append the given text line to the builder, indented by the running indentation of the builder.
	 */
	CORE_API void AppendLine(const FText& Text);

	/**
	 * Append the given string line to the builder, indented by the running indentation of the builder.
	 */
	CORE_API void AppendLine(const FString& String);

	/**
	 * Append the given name line to the builder, indented by the running indentation of the builder.
	 */
	CORE_API void AppendLine(const FName& Name);

	/**
	 * Append the given formatted text line to the builder, indented by the running indentation of the builder.
	 */
	CORE_API void AppendLineFormat(const FTextFormat& Pattern, const FFormatNamedArguments& Arguments);

	/**
	 * Append the given formatted text line to the builder, indented by the running indentation of the builder.
	 */
	CORE_API void AppendLineFormat(const FTextFormat& Pattern, const FFormatOrderedArguments& Arguments);

	/**
	 * Append the given formatted text line to the builder, indented by the running indentation of the builder.
	 */
	template <typename... ArgTypes>
	inline void AppendLineFormat(FTextFormat Pattern, ArgTypes... Args)
	{
		static_assert((TIsConstructible<FFormatArgumentValue, ArgTypes>::Value && ...), "Invalid argument type passed to FTextBuilder::AppendLineFormat");
		static_assert(sizeof...(Args) > 0, "FTextBuilder::AppendLineFormat expects at least one non-format argument"); // we do this to ensure that people don't call AppendLineFormat for no good reason

		BuildAndAppendLine(FText::Format(MoveTemp(Pattern), FFormatOrderedArguments{ MoveTemp(Args)... }));
	}

	/**
	 * Clear the builder and reset it to its default state.
	 */
	CORE_API void Clear();

	/**
	 * Check to see if the builder has any data.
	 */
	CORE_API bool IsEmpty() const;

	/**
	 * Returns the number of lines.
	 */
	CORE_API int32 GetNumLines() const;

	/**
	 * Build the current set of input into a FText.
	 */
	CORE_API FText ToText() const;

private:
	CORE_API void BuildAndAppendLine(FString&& Data);
	CORE_API void BuildAndAppendLine(FText&& Data);

	TArray<FText> Lines;
	int32 IndentCount = 0;
};

/** Unicode character helper functions (see FTextChar for more functionality) */
struct FUnicodeChar
{
	UE_DEPRECATED(5.7, "FUnicodeChar::CodepointToString is deprecated. Use FTextChar::ConvertCodepointToString instead.")
	static CORE_API bool CodepointToString(const uint32 InCodepoint, FString& OutString);
};

/**
 * Unicode Bidirectional text support 
 * http://www.unicode.org/reports/tr9/
 */
namespace TextBiDi
{
	/** Lists the potential reading directions for text */
	enum class ETextDirection : uint8
	{
		/** Contains only LTR text - requires simple LTR layout */
		LeftToRight,
		/** Contains only RTL text - requires simple RTL layout */
		RightToLeft,
		/** Contains both LTR and RTL text - requires more complex layout using multiple runs of text */
		Mixed,
	};

	/** A single complex layout entry. Defines the starting position, length, and reading direction for a sub-section of text */
	struct FTextDirectionInfo
	{
		int32 StartIndex;
		int32 Length;
		ETextDirection TextDirection;
	};

	/** Defines the interface for a re-usable BiDi object */
	class ITextBiDi
	{
	public:
		virtual ~ITextBiDi() = default;

		/** See TextBiDi::ComputeTextDirection */
		virtual ETextDirection ComputeTextDirection(const FText& InText) = 0;
		virtual ETextDirection ComputeTextDirection(const FString& InString) = 0;
		virtual ETextDirection ComputeTextDirection(const TCHAR* InString, const int32 InStringStartIndex, const int32 InStringLen) = 0;

		/** See TextBiDi::ComputeTextDirection */
		virtual ETextDirection ComputeTextDirection(const FText& InText, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo) = 0;
		virtual ETextDirection ComputeTextDirection(const FString& InString, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo) = 0;
		virtual ETextDirection ComputeTextDirection(const TCHAR* InString, const int32 InStringStartIndex, const int32 InStringLen, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo) = 0;

		/** See TextBiDi::ComputeBaseDirection */
		virtual ETextDirection ComputeBaseDirection(const FText& InText) = 0;
		virtual ETextDirection ComputeBaseDirection(const FString& InString) = 0;
		virtual ETextDirection ComputeBaseDirection(const TCHAR* InString, const int32 InStringStartIndex, const int32 InStringLen) = 0;
	};

	/**
	 * Create a re-usable BiDi object.
	 * This may yield better performance than the utility functions if you're performing a lot of BiDi requests, as this object can re-use allocated data between requests.
	 */
	CORE_API TUniquePtr<ITextBiDi> CreateTextBiDi();

	/**
	 * Utility function which will compute the reading direction of the given text.
	 * @note You may want to use the version that returns you the advanced layout data in the Mixed case.
	 * @return LeftToRight if all of the text is LTR, RightToLeft if all of the text is RTL, or Mixed if the text contains both LTR and RTL text.
	 */
	CORE_API ETextDirection ComputeTextDirection(const FText& InText);
	CORE_API ETextDirection ComputeTextDirection(const FString& InString);
	CORE_API ETextDirection ComputeTextDirection(const TCHAR* InString, const int32 InStringStartIndex, const int32 InStringLen);

	/**
	 * Utility function which will compute the reading direction of the given text, as well as populate any advanced layout data for the text.
	 * The base direction is the overall reading direction of the text (see ComputeBaseDirection). This will affect where some characters (such as brackets and quotes) are placed within the resultant FTextDirectionInfo data.
	 * @return LeftToRight if all of the text is LTR, RightToLeft if all of the text is RTL, or Mixed if the text contains both LTR and RTL text.
	 */
	CORE_API ETextDirection ComputeTextDirection(const FText& InText, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo);
	CORE_API ETextDirection ComputeTextDirection(const FString& InString, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo);
	CORE_API ETextDirection ComputeTextDirection(const TCHAR* InString, const int32 InStringStartIndex, const int32 InStringLen, const ETextDirection InBaseDirection, TArray<FTextDirectionInfo>& OutTextDirectionInfo);

	/**
	 * Utility function which will compute the base direction of the given text.
	 * This provides the text flow direction that should be used when combining bidirectional text runs together.
	 * @return RightToLeft if the first character in the string has a bidirectional character type of R or AL, otherwise LeftToRight.
	 */
	CORE_API ETextDirection ComputeBaseDirection(const FText& InText);
	CORE_API ETextDirection ComputeBaseDirection(const FString& InString);
	CORE_API ETextDirection ComputeBaseDirection(const TCHAR* InString, const int32 InStringStartIndex, const int32 InStringLen);

	/**
	 * Utility function which tests to see whether the given character is a bidirectional control character.
	 */
	CORE_API bool IsControlCharacter(const TCHAR InChar);
} // namespace TextBiDi

Expose_TNameOf(FText)
