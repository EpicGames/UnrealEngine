// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class FString;

/**
 * Utility for performing Unicode aware (in builds with ICU enabled) character checks.
 * In builds without ICU enabled (such as server builds), this API uses the non-Unicode aware FChar API internally.
 * The implementation can be found in LegacyText.cpp and ICUText.cpp.
 */
class FTextChar
{
public:
	/**
	 * Return the next codepoint from the given null terminated buffer.
	 */
	[[nodiscard]] static CORE_API UTF32CHAR GetCodepoint(const TCHAR* Buffer, int32* OutNumTCHARInCodepoint = nullptr);

	/**
	 * Attempt to convert the given codepoint to a string, replacing any existing content in the string.
	 */
	[[nodiscard]] static CORE_API bool ConvertCodepointToString(const UTF32CHAR Codepoint, FString& OutString);

	/**
	 * Attempt to convert the given codepoint to a string, appending after any existing content in the string.
	 */
	[[nodiscard]] static CORE_API bool AppendCodepointToString(const UTF32CHAR Codepoint, FString& OutString);

	/**
	 * Determine whether the given codepoint is a letter character.
	 */
	[[nodiscard]] static CORE_API bool IsAlpha(const UTF32CHAR Codepoint);

	/**
	 * Determine whether the given codepoint is a "graphic" character (printable, excluding spaces).
	 */
	[[nodiscard]] static CORE_API bool IsGraph(const UTF32CHAR Codepoint);

	/**
	 * Determine whether the given codepoint is a printable character.
	 */
	[[nodiscard]] static CORE_API bool IsPrint(const UTF32CHAR Codepoint);

	/**
	 * Determine whether the given codepoint is a punctuation character.
	 */
	[[nodiscard]] static CORE_API bool IsPunct(const UTF32CHAR Codepoint);

	/**
	 * Determine whether the given codepoint is an alphanumeric character (letter or digit).
	 */
	[[nodiscard]] static CORE_API bool IsAlnum(const UTF32CHAR Codepoint);

	/**
	 * Determine whether the given codepoint is a digit character.
	 */
	[[nodiscard]] static CORE_API bool IsDigit(const UTF32CHAR Codepoint);

	/**
	 * Determine whether the given codepoint is a hexadecimal digit character.
	 */
	[[nodiscard]] static CORE_API bool IsHexDigit(const UTF32CHAR Codepoint);

	/**
	 * Determine whether the given codepoint is a control character.
	 */
	[[nodiscard]] static CORE_API bool IsControl(const UTF32CHAR Codepoint);

	/**
	 * Determine whether the given codepoint is a whitespace character.
	 * @note It is safe to pass a UTF-16 TCHAR to this function, since whitespace is never a pair of UTF-16 characters.
	 */
	[[nodiscard]] static CORE_API bool IsWhitespace(const UTF32CHAR Codepoint);

	/**
	 * Determine whether the given codepoint is an uppercase letter character.
	 */
	[[nodiscard]] static CORE_API bool IsUpper(const UTF32CHAR Codepoint);

	/**
	 * Determine whether the given codepoint is an lowercase letter character.
	 */
	[[nodiscard]] static CORE_API bool IsLower(const UTF32CHAR Codepoint);

	/**
	 * Convert the given codepoint to its lowercase version.
	 */
	[[nodiscard]] static CORE_API UTF32CHAR ToLower(const UTF32CHAR Codepoint);

	/**
	 * Convert the given codepoint to its uppercase version.
	 */
	[[nodiscard]] static CORE_API UTF32CHAR ToUpper(const UTF32CHAR Codepoint);
};
