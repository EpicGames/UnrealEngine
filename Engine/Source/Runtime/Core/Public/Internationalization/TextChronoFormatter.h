// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Internationalization/Text.h"

/**
 * Utility for performing low-level localized chronological formats.
 * The implementation can be found in LegacyText.cpp and ICUText.cpp.
 */
class FTextChronoFormatter
{
public:
	static CORE_API FString AsDate(const FDateTime& DateTime, const EDateTimeStyle::Type DateStyle, const FString& TimeZone, const FCulture& TargetCulture);
	static CORE_API FString AsDateTime(const FDateTime& DateTime, const EDateTimeStyle::Type DateStyle, const EDateTimeStyle::Type TimeStyle, const FString& TimeZone, const FCulture& TargetCulture);
	static CORE_API FString AsDateTime(const FDateTime& DateTime, const FString& CustomPattern, const FString& TimeZone, const FCulture& TargetCulture);
	static CORE_API FString AsTime(const FDateTime& DateTime, const EDateTimeStyle::Type TimeStyle, const FString& TimeZone, const FCulture& TargetCulture);
};
