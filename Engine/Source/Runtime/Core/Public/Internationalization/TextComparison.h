// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"

namespace ETextComparisonLevel
{
	enum Type
	{
		Default,	// Locale-specific Default
		Primary,	// Base
		Secondary,	// Accent
		Tertiary,	// Case
		Quaternary,	// Punctuation
		Quinary		// Identical
	};
}

/**
 * Utility for performing low-level text comparison.
 * The implementation can be found in LegacyText.cpp and ICUText.cpp.
 */
class FTextComparison
{
public:
	static CORE_API int32 CompareTo(const FString& A, const FString& B, const ETextComparisonLevel::Type ComparisonLevel = ETextComparisonLevel::Default);
	static CORE_API int32 CompareToCaseIgnored(const FString& A, const FString& B);

	static CORE_API bool EqualTo(const FString& A, const FString& B, const ETextComparisonLevel::Type ComparisonLevel = ETextComparisonLevel::Default);
	static CORE_API bool EqualToCaseIgnored(const FString& A, const FString& B);
};
