// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/SortedMap.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Internationalization/ITextFormatArgumentModifier.h"
#include "Internationalization/Text.h"
#include "Misc/ExpressionParserTypes.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

struct FPrivateTextFormatArguments;

/**
 * Definition of the pattern used during a text format.
 */
class FTextFormatPatternDefinition : public TSharedFromThis<FTextFormatPatternDefinition, ESPMode::ThreadSafe>
{
public:
	/** Constructor */
	CORE_API FTextFormatPatternDefinition();

	/** Singleton access to the default instance */
	static CORE_API FTextFormatPatternDefinitionConstRef GetDefault();

	/** Get the text format definitions used when formatting text */
	CORE_API const FTokenDefinitions& GetTextFormatDefinitions() const;

	/** Is the given character one that an escape token may escape? */
	inline bool IsValidEscapeChar(const TCHAR InChar) const
	{
		return InChar == EscapeChar
			|| InChar == ArgStartChar
			|| InChar == ArgEndChar
			|| InChar == ArgModChar;
	}

	/** Is the given character one that should cause a literal string token to break parsing? */
	inline bool IsLiteralBreakChar(const TCHAR InChar) const
	{
		return InChar == EscapeChar
			|| InChar == ArgStartChar;
	}

	/** Character representing the start of an escape token */
	TCHAR EscapeChar = TEXT('`');
	FTextFormatPatternDefinition& SetEscapeChar(const TCHAR InChar) { EscapeChar = InChar; return *this; }

	/** Character representing the start of a format argument token */
	TCHAR ArgStartChar = TEXT('{');
	FTextFormatPatternDefinition& SetArgStartChar(const TCHAR InChar) { ArgStartChar = InChar; return *this; }

	/** Character representing the end of a format argument token */
	TCHAR ArgEndChar = TEXT('}');
	FTextFormatPatternDefinition& SetArgEndChar(const TCHAR InChar) { ArgEndChar = InChar; return *this; }

	/** Character representing the start of a format argument modifier token */
	TCHAR ArgModChar = TEXT('|');
	FTextFormatPatternDefinition& SetArgModChar(const TCHAR InChar) { ArgModChar = InChar; return *this; }

private:
	/** Token definitions for the text format lexer */
	FTokenDefinitions TextFormatDefinitions;
};

/**
 * A text formatter is responsible for formatting text patterns using a set of named or ordered arguments.
 */
class FTextFormatter
{
public:
	/** Callback function used to compile an argument modifier. Takes an argument modifier string and pattern definition, then returns the compiled result. */
	typedef TFunction<TSharedPtr<ITextFormatArgumentModifier>(const FTextFormatString&, FTextFormatPatternDefinitionConstRef)> FCompileTextArgumentModifierFuncPtr;

	/** Singleton access */
	static CORE_API FTextFormatter& Get();

	CORE_API void RegisterTextArgumentModifier(const FTextFormatString& InKeyword, FCompileTextArgumentModifierFuncPtr InCompileFunc);
	CORE_API void UnregisterTextArgumentModifier(const FTextFormatString& InKeyword);
	CORE_API FCompileTextArgumentModifierFuncPtr FindTextArgumentModifier(const FTextFormatString& InKeyword) const;

	/** Low-level versions of Format. You probably want to use FText::Format(...) rather than call these directly */
	static CORE_API FText Format(FTextFormat&& InFmt, FFormatNamedArguments&& InArguments, const bool bInRebuildText, const bool bInRebuildAsSource);
	static CORE_API FText Format(FTextFormat&& InFmt, FFormatOrderedArguments&& InArguments, const bool bInRebuildText, const bool bInRebuildAsSource);
	static CORE_API FText Format(FTextFormat&& InFmt, TArray<FFormatArgumentData>&& InArguments, const bool bInRebuildText, const bool bInRebuildAsSource);

	/** Low-level version of Format that returns a string. This should typically only be used externally when rebuilding the display string for some formatted text */
	static CORE_API FString FormatStr(const FTextFormat& InFmt, const FFormatNamedArguments& InArguments, const bool bInRebuildText, const bool bInRebuildAsSource);
	static CORE_API FString FormatStr(const FTextFormat& InFmt, const FFormatOrderedArguments& InArguments, const bool bInRebuildText, const bool bInRebuildAsSource);
	static CORE_API FString FormatStr(const FTextFormat& InFmt, const TArray<FFormatArgumentData>& InArguments, const bool bInRebuildText, const bool bInRebuildAsSource);

	/** Incredibly low-level version of format. You should only be calling this if you're implementing a custom argument modifier type that itself needs to format using the private arguments */
	static CORE_API FString Format(const FTextFormat& InFmt, const FPrivateTextFormatArguments& InFormatArgs);

	/** Incredibly low-level version of FFormatArgumentValue::ToFormattedString. You should only be calling this if you're implementing a custom argument modifier type that itself needs to convert the argument to a string */
	static CORE_API void ArgumentValueToFormattedString(const FFormatArgumentValue& InValue, const FPrivateTextFormatArguments& InFormatArgs, FString& OutResult);

private:
	static int32 EstimateArgumentValueLength(const FFormatArgumentValue& ArgumentValue);

	FTextFormatter();

	/** Functions for constructing argument modifier data */
	TSortedMap<FTextFormatString, FCompileTextArgumentModifierFuncPtr> TextArgumentModifiers;

	/** RW lock protecting the argument modifiers map from being modified concurrently */
	mutable FTransactionallySafeRWLock TextArgumentModifiersRW;
};
