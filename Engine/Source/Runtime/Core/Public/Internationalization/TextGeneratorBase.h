// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "ITextGenerator.h"
#include "Serialization/StructuredArchive.h"
#include "UObject/NameTypes.h"

/**
 * Base class implementation for ITextGenerator.
 */
class FTextGeneratorBase : public ITextGenerator
{
public: // Serialization
	/**
	 * Gets the type ID of this generator. The type ID is used to reconstruct this type for serialization and must be registered with FText::RegisterTextGenerator().
	 */
	CORE_API virtual FName GetTypeID() const override;

	/**
	 * Serializes this generator.
	 */
	CORE_API virtual void Serialize(FStructuredArchive::FRecord Record) override;
};
