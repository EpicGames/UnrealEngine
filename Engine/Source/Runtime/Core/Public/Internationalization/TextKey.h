// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/TypeHash.h"
#include "Containers/StringConv.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Serialization/StructuredArchive.h"

#ifndef UE_TEXTKEY_STORE_EMBEDDED_HASH
	#define UE_TEXTKEY_STORE_EMBEDDED_HASH (0)
#endif

namespace TextKeyUtil
{
	/** Utility to produce a hash for a UTF-16 string (as used by FTextKey) */
	CORE_API uint32 HashString(const FTCHARToUTF16& InStr);
	UE_FORCEINLINE_HINT uint32 HashString(const FTCHARToUTF16& InStr, const uint32 InBaseHash)
	{
		return HashCombine(HashString(InStr), InBaseHash);
	}

	/** Utility to produce a hash for a string (as used by FTextKey) */
	inline uint32 HashString(const TCHAR* InStr)
	{
		FTCHARToUTF16 UTF16String(InStr);
		return HashString(UTF16String);
	}
	inline uint32 HashString(const TCHAR* InStr, const uint32 InBaseHash)
	{
		FTCHARToUTF16 UTF16String(InStr);
		return HashString(UTF16String, InBaseHash);
	}

	/** Utility to produce a hash for a string (as used by FTextKey) */
	inline uint32 HashString(const TCHAR* InStr, const int32 InStrLen)
	{
		FTCHARToUTF16 UTF16String(InStr, InStrLen);
		return HashString(UTF16String);
	}
	inline uint32 HashString(const TCHAR* InStr, const int32 InStrLen, const uint32 InBaseHash)
	{
		FTCHARToUTF16 UTF16String(InStr, InStrLen);
		return HashString(UTF16String, InBaseHash);
	}

	/** Utility to produce a hash for a string (as used by FTextKey) */
	UE_FORCEINLINE_HINT uint32 HashString(const FString& InStr)
	{
		return HashString(*InStr, InStr.Len());
	}
	UE_FORCEINLINE_HINT uint32 HashString(const FString& InStr, const uint32 InBaseHash)
	{
		return HashString(*InStr, InStr.Len(), InBaseHash);
	}

	/** Utility to produce a hash for a string (as used by FTextKey) */
	UE_FORCEINLINE_HINT uint32 HashString(FStringView InStr)
	{
		return HashString(InStr.GetData(), InStr.Len());
	}
	UE_FORCEINLINE_HINT uint32 HashString(FStringView InStr, const uint32 InBaseHash)
	{
		return HashString(InStr.GetData(), InStr.Len(), InBaseHash);
	}
}

/**
 * Optimized representation of a case-sensitive string, as used by localization keys.
 * This references an entry within a internal table to avoid memory duplication, as well as offering optimized comparison and hashing performance.
 */
class FTextKey
{
public:
	FTextKey() = default;
	CORE_API FTextKey(FStringView InStr);
	CORE_API FTextKey(const TCHAR* InStr);
	CORE_API FTextKey(const FString& InStr);

	/** Convert this text key back to its string representation */
	CORE_API FString ToString() const;
	CORE_API void ToString(FString& Out) const;
	CORE_API void ToString(FStringBuilderBase& Out) const;
	CORE_API void AppendString(FString& Out) const;
	CORE_API void AppendString(FStringBuilderBase& Out) const;

	/** Compare for equality */
	friend UE_FORCEINLINE_HINT bool operator==(const FTextKey& A, const FTextKey& B)
	{
		return A.Index == B.Index;
	}

	/** Compare for inequality */
	friend UE_FORCEINLINE_HINT bool operator!=(const FTextKey& A, const FTextKey& B)
	{
		return A.Index != B.Index;
	}

	/** Get the hash of this text key */
	friend CORE_API uint32 GetTypeHash(const FTextKey& A);

	/** Serialize this text key as if it were an FString */
	CORE_API void SerializeAsString(FArchive& Ar);

	/** Serialize this text key including its hash value (this method is sensitive to hashing algorithm changes, so only use it for generated files that can be rebuilt from another source) */
	CORE_API void SerializeWithHash(FArchive& Ar);

	/** Serialize this text key including its hash value, discarding the hash on load (to upgrade from an older hashing algorithm) */
	CORE_API void SerializeDiscardHash(FArchive& Ar);

	/** Serialize this text key as if it were an FString */
	CORE_API void SerializeAsString(FStructuredArchiveSlot Slot);

	/** Serialize this text key including its hash value (this method is sensitive to hashing algorithm changes, so only use it for generated files that can be rebuilt from another source) */
	CORE_API void SerializeWithHash(FStructuredArchiveSlot Slot);

	/** Serialize this text key including its hash value, discarding the hash on load (to upgrade from an older hashing algorithm) */
	CORE_API void SerializeDiscardHash(FStructuredArchiveSlot Slot);

	/** Is this text key empty? */
	UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return Index == INDEX_NONE;
	}

	/** Reset this text key to be empty */
	inline void Reset()
	{
		Index = INDEX_NONE;
#if UE_TEXTKEY_STORE_EMBEDDED_HASH
		StrHash = 0;
#endif
	}

	/** Compact any slack within the internal table */
	static CORE_API void CompactDataStructures();

	/** Do not use any FTextKey or FTextId after calling this */
	static CORE_API void TearDown();

private:
	/** Index of the internal FKeyData we reference */
	int32 Index = INDEX_NONE;

#if UE_TEXTKEY_STORE_EMBEDDED_HASH
	/** Local cache of FKeyData::StrHash to avoid indirection into the internal table */
	uint32 StrHash = 0;
#endif

	friend class FTextKeyState;
};

/**
 * Optimized representation of a text identity (a namespace and key pair).
 */
class FTextId
{
public:
	FTextId() = default;

	FTextId(const FTextKey& InNamespace, const FTextKey& InKey)
		: Namespace(InNamespace)
		, Key(InKey)
	{
	}

	/** Get the namespace component of this text identity */
	UE_FORCEINLINE_HINT FTextKey GetNamespace() const
	{
		return Namespace;
	}

	/** Get the key component of this text identity */
	UE_FORCEINLINE_HINT FTextKey GetKey() const
	{
		return Key;
	}

	/** Compare for equality */
	friend UE_FORCEINLINE_HINT bool operator==(const FTextId& A, const FTextId& B)
	{
		return A.Namespace == B.Namespace && A.Key == B.Key;
	}

	/** Compare for inequality */
	friend UE_FORCEINLINE_HINT bool operator!=(const FTextId& A, const FTextId& B)
	{
		return A.Namespace != B.Namespace || A.Key != B.Key;
	}

	/** Get the hash of this text identity */
	friend UE_FORCEINLINE_HINT uint32 GetTypeHash(const FTextId& A)
	{
		return HashCombine(GetTypeHash(A.Namespace), GetTypeHash(A.Key));
	}

	/** Serialize this text identity as if it were FStrings */
	void SerializeAsString(FArchive& Ar)
	{
		Namespace.SerializeAsString(Ar);
		Key.SerializeAsString(Ar);
	}

	/** Serialize this text identity including its hash values (this method is sensitive to hashing algorithm changes, so only use it for generated files that can be rebuilt from another source) */
	void SerializeWithHash(FArchive& Ar)
	{
		Namespace.SerializeWithHash(Ar);
		Key.SerializeWithHash(Ar);
	}

	/** Serialize this text identity including its hash values, discarding the hash on load (to upgrade from an older hashing algorithm) */
	void SerializeDiscardHash(FArchive& Ar)
	{
		Namespace.SerializeDiscardHash(Ar);
		Key.SerializeDiscardHash(Ar);
	}

	/** Serialize this text identity as if it were FStrings */
	void SerializeAsString(FStructuredArchiveSlot Slot)
	{
		FStructuredArchiveRecord Record = Slot.EnterRecord();

		Namespace.SerializeAsString(Record.EnterField(TEXT("Namespace")));
		Key.SerializeAsString(Record.EnterField(TEXT("Key")));
	}

	/** Serialize this text identity including its hash values (this method is sensitive to hashing algorithm changes, so only use it for generated files that can be rebuilt from another source) */
	void SerializeWithHash(FStructuredArchiveSlot Slot)
	{
		FStructuredArchiveRecord Record = Slot.EnterRecord();

		Namespace.SerializeWithHash(Record.EnterField(TEXT("Namespace")));
		Key.SerializeWithHash(Record.EnterField(TEXT("Key")));
	}

	/** Serialize this text identity including its hash values, discarding the hash on load (to upgrade from an older hashing algorithm) */
	void SerializeDiscardHash(FStructuredArchiveSlot Slot)
	{
		FStructuredArchiveRecord Record = Slot.EnterRecord();

		Namespace.SerializeDiscardHash(Record.EnterField(TEXT("Namespace")));
		Key.SerializeDiscardHash(Record.EnterField(TEXT("Key")));
	}

	/** Is this text identity empty? */
	UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return Namespace.IsEmpty() && Key.IsEmpty();
	}

	/** Reset this text identity to be empty */
	inline void Reset()
	{
		Namespace.Reset();
		Key.Reset();
	}

private:
	FTextKey Namespace;
	FTextKey Key;
};
