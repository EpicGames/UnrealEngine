// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphFwd.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"
#include "Internationalization/LocKeyFuncs.h"
#include "Internationalization/LocTesting.h"
#include "Internationalization/LocalizedTextSourceTypes.h"
#include "Internationalization/TextKey.h"
#include "Misc/Crc.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

#include <atomic>

class FTextLocalizationResource;
class ILocalizedTextSource;
class IPakFile;
struct FLogCategoryBase;
struct FPolyglotTextData;

enum class ETextLocalizationManagerInitializedFlags : uint8
{
	None = 0,
	Engine = 1<<0,
	Game = 1<<1,
};
ENUM_CLASS_FLAGS(ETextLocalizationManagerInitializedFlags);

/** Singleton class that manages display strings for FText. */
class FTextLocalizationManager
{
	friend CORE_API void BeginPreInitTextLocalization();
	friend CORE_API void BeginInitTextLocalization();
	friend CORE_API void InitEngineTextLocalization();
	friend CORE_API void InitGameTextLocalization();

private:

	/** Data struct for tracking a display string. */
	struct FDisplayStringEntry
	{
		FTextConstDisplayStringRef DisplayString;
#if WITH_EDITORONLY_DATA
		FTextKey LocResID;
#endif
		int32 LocalizationTargetPathId = INDEX_NONE;
		uint32 SourceStringHash;

		FDisplayStringEntry(const FTextKey& InLocResID, const int32 InLocalizationTargetPathId, const uint32 InSourceStringHash, const FTextConstDisplayStringRef& InDisplayString)
			: DisplayString(InDisplayString)
#if WITH_EDITORONLY_DATA
			, LocResID(InLocResID)
#endif
			, LocalizationTargetPathId(InLocalizationTargetPathId)
			, SourceStringHash(InSourceStringHash)
		{
		}
	};

	/** Manages the currently loaded or registered text localizations. */
	typedef TMap<FTextId, FDisplayStringEntry> FDisplayStringLookupTable;

	struct FDisplayStringsForLocalizationTarget
	{
		/**
		 * The path of this localization target.
		 */
		FString LocalizationTargetPath;

		/**
		 * Text IDs currently associated with this localization target.
		 * @note This information is also known via FDisplayStringEntry::LocalizationTargetPathId, but this serves as 
		 *       an accelerator for HandleLocalizationTargetsUnmounted to avoid spinning the entire live table.
		 */
		TSet<FTextId> TextIds;

		/**
		 * True if this localization target has been mounted via HandleLocalizationTargetsMounted.
		 * @note Only mounted localization targets track TextIds, as they're the only things that can be unloaded via HandleLocalizationTargetsUnmounted.
		 */
		bool bIsMounted = false;
	};

	struct FDisplayStringsByLocalizationTargetId
	{
	public:
		FDisplayStringsForLocalizationTarget& FindOrAdd(FStringView InLocalizationTargetPath, int32* OutLocalizationTargetPathId = nullptr);
		FDisplayStringsForLocalizationTarget* Find(const int32 InLocalizationTargetPathId);
		void TrackTextId(const int32 InCurrentLocalizationPathId, const int32 InNewLocalizationPathId, const FTextId& InTextId);

	private:
		TArray<FDisplayStringsForLocalizationTarget> LocalizationTargets;
		TMap<FStringView, int32> LocalizationTargetPathsToIds;
	};

private:
	std::atomic<ETextLocalizationManagerInitializedFlags> InitializedFlags{ ETextLocalizationManagerInitializedFlags::None };
	
	bool IsInitialized() const
	{
		return InitializedFlags != ETextLocalizationManagerInitializedFlags::None;
	}

	mutable FRWLock DisplayStringTableRW;
	FDisplayStringLookupTable DisplayStringLookupTable;
	FDisplayStringsByLocalizationTargetId DisplayStringsByLocalizationTargetId;

	mutable FRWLock TextRevisionRW;
	TMap<FTextId, uint16> LocalTextRevisions;
	uint16 TextRevisionCounter;

#if WITH_EDITOR
	uint8 GameLocalizationPreviewAutoEnableCount;
	bool bIsGameLocalizationPreviewEnabled;
	bool bIsLocalizationLocked;
#endif

	CORE_API FTextLocalizationManager();
	friend class FLazySingleton;
	
public:

	CORE_API ~FTextLocalizationManager();
	UE_NONCOPYABLE(FTextLocalizationManager);

	/** Singleton accessor */
	static CORE_API FTextLocalizationManager& Get();
	static CORE_API void TearDown();

	static CORE_API bool IsDisplayStringSupportEnabled();

	CORE_API void DumpMemoryInfo() const;
	CORE_API void CompactDataStructures();

#if ENABLE_LOC_TESTING
	/**
	 * Dumps the current live table state to the log, optionally filtering it based on the given wildcard arguments.
	 * @note Calling this function with no filters specified will dump the entire live table.
	 */
private:
	void DumpLiveTableImpl(const FString* NamespaceFilter, const FString* KeyFilter, const FString* DisplayStringFilter, TFunctionRef<void(const FTextId& Id, const FTextConstDisplayStringRef& DisplayString)> Callback) const;
public:
	CORE_API void DumpLiveTable(const FString* NamespaceFilter = nullptr, const FString* KeyFilter = nullptr, const FString* DisplayStringFilter = nullptr, const FLogCategoryBase* CategoryOverride = nullptr) const;
	CORE_API void DumpLiveTable(const FString& OutputFilename, const FString* NamespaceFilter = nullptr, const FString* KeyFilter = nullptr, const FString* DisplayStringFilter = nullptr) const;
	CORE_API void AddOrUpdateDisplayStringInLiveTable(const FString& Namespace, const FString& Key, const FString& DisplayString, const FString* const SourceStringPtr = nullptr);
#endif

	/**
	 * Get the language that will be requested during localization initialization, based on the hierarchy of: command line -> configs -> OS default.
	 */
	CORE_API FString GetRequestedLanguageName() const;

	/**
	 * Get the locale that will be requested during localization initialization, based on the hierarchy of: command line -> configs -> OS default.
	 */
	CORE_API FString GetRequestedLocaleName() const;

	/**
	 * Given a localization category, get the native culture for the category (if known).
	 * @return The native culture for the given localization category, or an empty string if the native culture is unknown.
	 */
	CORE_API FString GetNativeCultureName(const ELocalizedTextSourceCategory InCategory) const;

	/**
	 * Get a list of culture names that we have localized resource data for (ELocalizationLoadFlags controls which resources should be checked).
	 */
	CORE_API TArray<FString> GetLocalizedCultureNames(const ELocalizationLoadFlags InLoadFlags) const;

	/**
	 * Get the language that we last loaded localization data for.
	 * eg) If the available languages were [en, fr, de] and the current language was "en-US", this function would return "en".
	 */
	CORE_API FString GetLocalizedLanguageName() const;

	/**
	 * Given a localization target path, get the ID associated with it.
	 * @note This ID is unstable and should only be used for quick in-process comparison.
	 */
	CORE_API int32 GetLocalizationTargetPathId(FStringView InLocalizationTargetPath);

	/**
	 * Register a localized text source with the text localization manager.
	 */
	CORE_API void RegisterTextSource(const TSharedRef<ILocalizedTextSource>& InLocalizedTextSource, const bool InRefreshResources = true);

	/**
	 * Register a polyglot text data with the text localization manager.
	 */
	CORE_API void RegisterPolyglotTextData(const FPolyglotTextData& InPolyglotTextData, const bool InAddDisplayString = true);
	CORE_API void RegisterPolyglotTextData(TArrayView<const FPolyglotTextData> InPolyglotTextDataArray, const bool InAddDisplayStrings = true);

	/**
	 * Finds and returns the display string with the given namespace and key, if it exists.
	 * @note If a non-null and non-empty source string is specified and the found localized display string was not localized from that source string, null will be returned.
	 */
	CORE_API FTextConstDisplayStringPtr FindDisplayString(const FTextKey& Namespace, const FTextKey& Key, const FString* const SourceStringPtr = nullptr) const;

	/**
	 * Get the current display string for the given namespace and key, if any.
	 * @note If a non-null and non-empty source string is specified and the found localized display string was not localized from that source string, it will be considered unlocalized.
	 * 
	 * Unlike FindDisplayString:
	 *   * This function may adjust the given text ID (eg, when USE_STABLE_LOCALIZATION_KEYS is enabled).
	 *   * This function may return a value for unlocalized strings (eg, when using -LEETifyUnlocalized).
	 */
	CORE_API FTextConstDisplayStringPtr GetDisplayString(const FTextKey& Namespace, const FTextKey& Key, const FString* const SourceStringPtr) const;

#if WITH_EDITORONLY_DATA
	/** If an entry exists for the specified namespace and key, returns true and provides the localization resource identifier from which it was loaded. Otherwise, returns false. */
	CORE_API bool GetLocResID(const FTextKey& Namespace, const FTextKey& Key, FString& OutLocResId) const;
#endif

	/** Updates display string entries and adds new display string entries based on localizations found in a specified localization resource. */
	CORE_API void UpdateFromLocalizationResource(const FString& LocalizationResourceFilePath);
	CORE_API void UpdateFromLocalizationResource(const FTextLocalizationResource& TextLocalizationResource);

	/**
	 * Wait for any current async tasks to finish.
	 * @see NotifyWhenAsyncTasksCompleted for an non-blocking version.
	 * 
	 * Async tasks will be started for anything that loads localization data (eg, initialization, language changes, requests to refresh the localization data, requests 
	 * to load additional localization data for chunked targets, or requests to load additional localization data for explicitly loaded plugins).
	 * 
	 * While the engine automatically waits at certain points during initialization, you may find that you need to add your own waits if you cause additional localization 
	 * data to load post-initialization (eg, because you're using chunked localization targets or are explicitly loading plugins). A good place to add a wait in that case 
	 * might be at the end of your loading screen, before showing your main menu or game world.
	 */
	CORE_API void WaitForAsyncTasks();

	/**
	 * Call the given function when any current async tasks have finished.
	 * @note The notification may be called from any thread, to avoid blocking other async tasks in the queue while waiting for a given thread to be available.
	 * @see WaitForAsyncTasks for a blocking version.
	 */
	CORE_API void NotifyWhenAsyncTasksCompleted(TUniqueFunction<void()>&& Notification);

	/**
	 * Reloads resources for the current culture.
	 * @note The loading is async, see WaitForAsyncTasks/NotifyWhenAsyncTasksCompleted, or pass the optional notification callback.
	 */
	CORE_API void RefreshResources(TUniqueFunction<void()>&& Notification = nullptr);

	/**
	 * Called when paths containing additional localization target data (LocRes) are mounted, to allow the display strings to dynamically update without waiting for a refresh.
	 * @note The loading is async, see WaitForAsyncTasks/NotifyWhenAsyncTasksCompleted, or pass the optional notification callback.
	 * @see FCoreDelegates::GatherAdditionalLocResPathsCallback.
	 */
	CORE_API void HandleLocalizationTargetsMounted(TArrayView<const FString> LocalizationTargetPaths, TUniqueFunction<void()>&& Notification = nullptr);

	 /**
	  * Called when paths containing additional localization target data (LocRes) are unmounted, to allow the display strings to be unloaded.
	  * @note The unloading is async, see WaitForAsyncTasks/NotifyWhenAsyncTasksCompleted, or pass the optional notification callback.
	  * @see FCoreDelegates::GatherAdditionalLocResPathsCallback.
	  */
	CORE_API void HandleLocalizationTargetsUnmounted(TArrayView<const FString> LocalizationTargetPaths, TUniqueFunction<void()>&& Notification = nullptr);

	/**
	 * Returns the current text revision number. This value can be cached when caching information from the text localization manager.
	 * If the revision does not match, cached information may be invalid and should be recached.
	 */
	CORE_API uint16 GetTextRevision() const;

	/**
	 * Attempts to find a local revision history for the given text ID.
	 * This will only be set if the display string has been changed since the localization manager version has been changed (eg, if it has been edited while keeping the same key).
	 * @return The local revision, or 0 if there have been no changes since a global history change.
	 */
	CORE_API uint16 GetLocalRevisionForTextId(const FTextId& InTextId) const;

	/**
	 * Get both the global and local revision for the given text ID.
	 * @see GetTextRevision and GetLocalRevisionForTextId.
	 */
	CORE_API void GetTextRevisions(const FTextId& InTextId, uint16& OutGlobalTextRevision, uint16& OutLocalTextRevision) const;

#if WITH_EDITOR
	/**
	 * Enable the game localization preview using the current "preview language" setting, or the native culture if no "preview language" is set.
	 * @note This is the same as calling EnableGameLocalizationPreview with the current "preview language" setting.
	 */
	CORE_API void EnableGameLocalizationPreview();

	/**
	 * Enable the game localization preview using the given language, or the native language if the culture name is empty.
	 * @note This will also lockdown localization editing if the given language is a non-native game language (to avoid accidentally baking out translations as source data in assets).
	 */
	CORE_API void EnableGameLocalizationPreview(const FString& CultureName);

	/**
	 * Disable the game localization preview.
	 * @note This is the same as calling EnableGameLocalizationPreview with the native game language (or an empty string).
	 */
	CORE_API void DisableGameLocalizationPreview();

	/**
	 * Is the game localization preview enabled for a non-native language?
	 */
	CORE_API bool IsGameLocalizationPreviewEnabled() const;

	/**
	 * Notify that the game localization preview should automatically enable itself under certain circumstances 
	 * (such as changing the preview language via the UI) due to a state change (such as PIE starting).
	 * @note This must be paired with a call to PopAutoEnableGameLocalizationPreview.
	 */
	CORE_API void PushAutoEnableGameLocalizationPreview();

	/**
	 * Notify that the game localization preview should no longer automatically enable itself under certain circumstances 
	 * (such as changing the preview language via the UI) due to a state change (such as PIE ending).
	 * @note This must be paired with a call to PushAutoEnableGameLocalizationPreview.
	 */
	CORE_API void PopAutoEnableGameLocalizationPreview();

	/**
	 * Should the game localization preview automatically enable itself under certain circumstances?
	 */
	CORE_API bool ShouldGameLocalizationPreviewAutoEnable() const;

	/**
	 * Configure the "preview language" setting used for the game localization preview.
	 */
	CORE_API void ConfigureGameLocalizationPreviewLanguage(const FString& CultureName);

	/**
	 * Get the configured "preview language" setting used for the game localization preview (if any).
	 */
	CORE_API FString GetConfiguredGameLocalizationPreviewLanguage() const;

	/**
	 * Is the localization of this game currently locked? (ie, can it be edited in the UI?).
	 */
	CORE_API bool IsLocalizationLocked() const;
#endif

	/**
	 * True if we should force load game localization data.
	 */
	CORE_API bool ShouldForceLoadGameLocalization() const;

	/** Event type for immediately reacting to changes in display strings for text. */
	DECLARE_EVENT(FTextLocalizationManager, FTextRevisionChangedEvent)
	FTextRevisionChangedEvent OnTextRevisionChangedEvent;

private:
	/** Callback for when a PAK file is loaded. Async loads any chunk specific localization resources. */
	void OnPakFileMounted(const IPakFile& PakFile);

	/** Callback for changes in culture. Async loads the new culture's localization resources. */
	void OnCultureChanged();

	/** Loads localization resources for the specified culture, optionally loading localization resources that are editor-specific or game-specific. */
	void LoadLocalizationResourcesForCulture_Sync(TArrayView<const TSharedPtr<ILocalizedTextSource>> AvailableTextSources, const FString& CultureName, const ELocalizationLoadFlags LocLoadFlags);
	void LoadLocalizationResourcesForCulture_Async(const FString& CultureName, const ELocalizationLoadFlags LocLoadFlags, TUniqueFunction<void()>&& Notification);

	/** Loads localization resources for the specified prioritized cultures, optionally loading localization resources that are editor-specific or game-specific. */
	void LoadLocalizationResourcesForPrioritizedCultures_Sync(TArrayView<const TSharedPtr<ILocalizedTextSource>> AvailableTextSources, TArrayView<const FString> PrioritizedCultureNames, const ELocalizationLoadFlags LocLoadFlags);
	void LoadLocalizationResourcesForPrioritizedCultures_Async(TArrayView<const FString> PrioritizedCultureNames, const ELocalizationLoadFlags LocLoadFlags, TUniqueFunction<void()>&& Notification);

	/** Loads the specified localization targets for the specified prioritized cultures, optionally loading localization resources that are editor-specific or game-specific. */
	void LoadLocalizationTargetsForPrioritizedCultures_Sync(TArrayView<const TSharedPtr<ILocalizedTextSource>> AvailableTextSources, TArrayView<const FString> LocalizationTargetPaths, TArrayView<const FString> PrioritizedCultureNames, const ELocalizationLoadFlags LocLoadFlags);
	void LoadLocalizationTargetsForPrioritizedCultures_Async(TArrayView<const FString> LocalizationTargetPaths, TArrayView<const FString> PrioritizedCultureNames, const ELocalizationLoadFlags LocLoadFlags, TUniqueFunction<void()>&& Notification);

	/** Loads the chunked localization resource data for the specified chunk */
	void LoadChunkedLocalizationResources_Sync(TArrayView<const TSharedPtr<ILocalizedTextSource>> AvailableTextSources, const int32 ChunkId, const FString& PakFilename);
	void LoadChunkedLocalizationResources_Async(const int32 ChunkId, const FString& PakFilename);

	/** Queue the given task to run async (where possible), chained to any existing AsyncLocalizationTask */
	void QueueAsyncTask(TUniqueFunction<void()>&& Task);

	struct FUpdateLiveTableOptions
	{
		FUpdateLiveTableOptions()
		{
		}

		bool bDirtyTextRevision = true;
		bool bReplaceExisting = true;
	};

	/** Updates display string entries and adds new display string entries. */
	void UpdateLiveTable(FTextLocalizationResource&& TextLocalizationResource, const FUpdateLiveTableOptions& UpdateOptions = FUpdateLiveTableOptions());

	/** Dirties the local revision counter for the given text ID by incrementing it (or adding it) */
	void DirtyLocalRevisionForTextId(const FTextId& InTextId);

	/** Internal version of FindDisplayString, shared between FindDisplayString and GetDisplayString */
	FTextConstDisplayStringPtr FindDisplayString_Internal(const FTextId& TextId, const FString& SourceString) const;

	/** Dirties the text revision counter by incrementing it, causing a revision mismatch for any information cached before this happens.  */
	void DirtyTextRevision();

	/** Cache the language that we last loaded localization data for. */
	void CacheLocalizedLanguageName(const ELocalizationLoadFlags InLoadFlags);

	/** Array of registered localized text sources, sorted by priority (@see RegisterTextSource) */
	TArray<TSharedPtr<ILocalizedTextSource>> LocalizedTextSources;

	/** The LocRes text source (this is also added to LocalizedTextSources, but we keep a pointer to it directly so we can patch in chunked LocRes data at runtime) */
	TSharedPtr<class FLocalizationResourceTextSource> LocResTextSource;

	/** The polyglot text source (this is also added to LocalizedTextSources, but we keep a pointer to it directly so we can add new polyglot data to it at runtime) */
	TSharedPtr<class FPolyglotTextSource> PolyglotTextSource;

	/** The latest async localization task (if any). Additional requests are chained to this so that they run in sequence */
	FGraphEventRef AsyncLocalizationTask;

	/** The language that we last loaded localization data for. */
	FString LocalizedLanguageName;
};
