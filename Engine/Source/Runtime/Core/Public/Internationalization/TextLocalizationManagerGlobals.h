// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

CORE_API void BeginPreInitTextLocalization();
CORE_API void BeginInitTextLocalization();
CORE_API void InitEngineTextLocalization();
CORE_API void InitGameTextLocalization();
