// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/SortedMap.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Internationalization/LocalizedTextSourceTypes.h"
#include "Internationalization/TextKey.h"
#include "Misc/Crc.h"
#include "Templates/SharedPointer.h"

class FArchive;

/** Utility class for working with Localization MetaData Resource (LocMeta) files. */
class FTextLocalizationMetaDataResource
{
public:
	FTextLocalizationMetaDataResource() = default;

	FTextLocalizationMetaDataResource(const FTextLocalizationMetaDataResource&) = default;
	FTextLocalizationMetaDataResource& operator=(const FTextLocalizationMetaDataResource&) = default;

	FTextLocalizationMetaDataResource(FTextLocalizationMetaDataResource&&) = default;
	FTextLocalizationMetaDataResource& operator=(FTextLocalizationMetaDataResource&&) = default;

	/** Name of the native culture for the localization target, eg) "en". */
	FString NativeCulture;

	/** Relative path to the native LocRes file for the localization target, eg) "en/Game.locres". */
	FString NativeLocRes;

	/** Name of all cultures with compiled LocRes files for the localization target (@note may be empty for older LocMeta files). */
	TArray<FString> CompiledCultures;

	/** True if this LocMeta was generated from UGC localization data (@see UserGeneratedContentLocalization) */
	bool bIsUGC = false;

	/** Load the given LocMeta file into this resource. */
	CORE_API bool LoadFromFile(const FString& FilePath);

	/** Load the given LocMeta archive into this resource. */
	CORE_API bool LoadFromArchive(FArchive& Archive, const FString& LocMetaID);

	/** Save this resource to the given LocMeta file. */
	CORE_API bool SaveToFile(const FString& FilePath);

	/** Save this resource to the given LocMeta archive. */
	CORE_API bool SaveToArchive(FArchive& Archive, const FString& LocMetaID);
};

/** Utility class for working with Localization Resource (LocRes) files. */
class FTextLocalizationResource
{
public:
	FTextLocalizationResource() = default;

	FTextLocalizationResource(const FTextLocalizationResource&) = default;
	FTextLocalizationResource& operator=(const FTextLocalizationResource&) = default;

	FTextLocalizationResource(FTextLocalizationResource&&) = default;
	FTextLocalizationResource& operator=(FTextLocalizationResource&&) = default;

	/** Data struct for tracking a localization entry from a localization resource. */
	struct FEntry
	{
		FTextConstDisplayStringPtr LocalizedString;
		FTextKey LocResID;
		int32 LocalizationTargetPathId = INDEX_NONE;
		uint32 SourceStringHash = 0;
		int32 Priority = 0; // Smaller numbers are higher priority
	};

	typedef TMap<FTextId, FEntry> FEntriesTable;
	FEntriesTable Entries;

	/** Utility to produce a hash for a string (as used by SourceStringHash) */
	static UE_FORCEINLINE_HINT uint32 HashString(const TCHAR* InStr, const uint32 InBaseHash = 0)
	{
		return FCrc::StrCrc32<TCHAR>(InStr, InBaseHash);
	}

	/** Utility to produce a hash for a string (as used by SourceStringHash) */
	static UE_FORCEINLINE_HINT uint32 HashString(const FString& InStr, const uint32 InBaseHash = 0)
	{
		return FCrc::StrCrc32<TCHAR>(*InStr, InBaseHash);
	}

	/** Add a single entry to this resource. */
	CORE_API void AddEntry(const FTextKey& InNamespace, const FTextKey& InKey, const FString& InSourceString, const FString& InLocalizedString, const int32 InPriority, const FTextKey& InLocResID = FTextKey());
	CORE_API void AddEntry(const FTextKey& InNamespace, const FTextKey& InKey, const FString& InSourceString, const FTextConstDisplayStringRef& InLocalizedString, const int32 InPriority, const FTextKey& InLocResID = FTextKey());
	CORE_API void AddEntry(const FTextKey& InNamespace, const FTextKey& InKey, const uint32 InSourceStringHash, const FString& InLocalizedString, const int32 InPriority, const FTextKey& InLocResID = FTextKey());
	CORE_API void AddEntry(const FTextKey& InNamespace, const FTextKey& InKey, const uint32 InSourceStringHash, const FTextConstDisplayStringRef& InLocalizedString, const int32 InPriority, const FTextKey& InLocResID = FTextKey());
	CORE_API void AddEntry(const FTextKey& InNamespace, const FTextKey& InKey, const FString& InSourceString, const FString& InLocalizedString, const int32 InPriority, const FTextKey& InLocResID, const int32 InLocalizationTargetPathID);
	CORE_API void AddEntry(const FTextKey& InNamespace, const FTextKey& InKey, const uint32 InSourceStringHash, const FTextConstDisplayStringRef& InLocalizedString, const int32 InPriority, const FTextKey& InLocResID, const int32 InLocalizationTargetPathID);
	/** Is this resource empty? */
	CORE_API bool IsEmpty() const;

	/** Load all LocRes files in the specified directory into this resource. */
	CORE_API void LoadFromDirectory(const FString& DirectoryPath, const int32 Priority);

	/** Load the given LocRes file into this resource. */
	CORE_API bool LoadFromFile(const FString& FilePath, const int32 Priority);

	/** Load the given LocRes archive into this resource. */
	CORE_API bool LoadFromArchive(FArchive& Archive, const FTextKey& LocResID, const int32 Priority);

	/** Save this resource to the given LocRes file. */
	CORE_API bool SaveToFile(const FString& FilePath) const;

	/** Save this resource to the given LocRes archive. */
	CORE_API bool SaveToArchive(FArchive& Archive, const FTextKey& LocResID) const;

private:
	/** Test whether the new entry should replace the current entry, optionally logging a conflict */
	static CORE_API bool ShouldReplaceEntry(const FTextKey& Namespace, const FTextKey& Key, const FEntry& CurrentEntry, const FEntry& NewEntry);
};

namespace TextLocalizationResourceUtil
{

/**
 * Given some paths to look at, get the native culture for the targets within those paths (if known).
 * @return The native culture for the targets within the given paths based on the data in the first LocMeta file, or an empty string if the native culture is unknown.
 */
CORE_API FString GetNativeCultureName(const TArray<FString>& InLocalizationPaths);

/**
 * Given a localization category, get the native culture for the targets for that category (if known).
 * @return The native culture for the given localization category, or an empty string if the native culture is unknown.
 */
CORE_API FString GetNativeCultureName(const ELocalizedTextSourceCategory InCategory);

/**
 * Get the native culture for the current project (if known).
 * @return The native culture for the current project based on the data in the game LocMeta files, or an empty string if the native culture is unknown.
 */
CORE_API FString GetNativeProjectCultureName(const bool bSkipCache = false);

/**
 * Clear the native culture for the current project so it will be re-cached on the text call to GetNativeProjectCultureName.
 */
CORE_API void ClearNativeProjectCultureName();

/**
 * Get the native culture for the engine.
 * @return The native culture for the engine based on the data in the engine LocMeta files.
 */
CORE_API FString GetNativeEngineCultureName(const bool bSkipCache = false);

/**
 * Clear the native culture for the engine so it will be re-cached on the text call to GetNativeEngineCultureName.
 */
CORE_API void ClearNativeEngineCultureName();

#if WITH_EDITOR
/**
 * Get the native culture for the editor.
 * @return The native culture for the editor based on the data in the editor LocMeta files.
 */
CORE_API FString GetNativeEditorCultureName(const bool bSkipCache = false);

/**
 * Clear the native culture for the editor so it will be re-cached on the text call to GetNativeEditorCultureName.
 */
CORE_API void ClearNativeEditorCultureName();
#endif	// WITH_EDITOR

/**
 * Given some paths to look at, populate a list of culture names that we have available localization resource information for.
 */
CORE_API TArray<FString> GetLocalizedCultureNames(const TArray<FString>& InLocalizationPaths);

/**
 * Get the array of localization targets that have been disabled for the current configuration.
 */
CORE_API const TArray<FString>& GetDisabledLocalizationTargets();

/**
 * Get the name that the given localization target should have for the given chunk ID.
 */
CORE_API FString GetLocalizationTargetNameForChunkId(const FString& InLocalizationTargetName, const int32 InChunkId);

/**
 * Derive the target path ID from the given text key.
 */
CORE_API int32 GetLocalizationTargetPathIdFromLocResId(const FTextKey& InLocResID);
}
