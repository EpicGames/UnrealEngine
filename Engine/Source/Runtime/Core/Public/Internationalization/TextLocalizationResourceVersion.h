// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreTypes.h"
#include "Misc/Guid.h"

/** Version information for compiled LocMeta (Localization MetaData Resource) and LocRes (Localization Resource) files */
struct FTextLocalizationResourceVersion
{
	/**
	 * Magic number identifying a LocMeta file.
	 */
	static CORE_API const FGuid LocMetaMagic;

	/**
	 * Magic number identifying a LocRes file.
	 * @note Legacy LocRes files will be missing this as it wasn't added until version 1.
	 */
	static CORE_API const FGuid LocResMagic;

	/**
	 * Data versions for LocMeta files.
	 */
	enum class ELocMetaVersion : uint8
	{
		/** Initial format. */
		Initial = 0,
		/** Added complete list of cultures compiled for the localization target. */
		AddedCompiledCultures,
		/** Added bIsUGC flag */
		AddedIsUGC,

		LatestPlusOne,
		Latest = LatestPlusOne - 1
	};

	/**
	 * Data versions for LocRes files.
	 */
	enum class ELocResVersion : uint8
	{
		/** Legacy format file - will be missing the magic number. */
		Legacy = 0,
		/** Compact format file - strings are stored in a LUT to avoid duplication. */
		Compact,
		/** Optimized format file - namespaces/keys are pre-hashed (CRC32), we know the number of elements up-front, and the number of references for each string in the LUT (to allow stealing). */
		Optimized_CRC32,
		/** Optimized format file - namespaces/keys are pre-hashed (CityHash64, UTF-16), we know the number of elements up-front, and the number of references for each string in the LUT (to allow stealing). */
		Optimized_CityHash64_UTF16,

		LatestPlusOne,
		Latest = LatestPlusOne - 1
	};
};
