// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// USE_STABLE_LOCALIZATION_KEYS controls whether we attempt to keep localization keys stable when editing text properties.
// This requires WITH_EDITORONLY_DATA, but can also be optionally disabled in editor builds by changing its value to 0.
// Note: Prior to this flag being introduced, localization package namespace (loc package ids) did not exist. Therefore, any
// code related to the use of localization package namespace should be wrapped around this macro. The whole goal of a loc
// package namespace is all about stabilizing text localization keys within a package.
#if WITH_EDITORONLY_DATA
	#define USE_STABLE_LOCALIZATION_KEYS (1)
#else	// WITH_EDITORONLY_DATA
	#define USE_STABLE_LOCALIZATION_KEYS (0)
#endif	// WITH_EDITORONLY_DATA
