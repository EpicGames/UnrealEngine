// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextNamespaceFwd.h"

class FArchive;

namespace TextNamespaceUtil
{

constexpr inline TCHAR PackageNamespaceStartMarker = TEXT('[');
constexpr inline TCHAR PackageNamespaceEndMarker = TEXT(']');

/**
 * Given a text and package namespace, build the full version that should be used by the localization system.
 * This can also be used to "zero-out" the package namespace used by a text namespace (by passing an empty package namespace) while still leaving the package namespace markers in place.
 *
 * @param InTextNamespace				The namespace currently used by the FText instance.
 * @param InPackageNamespace			The namespace of the package owning the FText instance.
 * @param bAlwaysApplyPackageNamespace	If true, this will always apply the package namespace to the text namespace.
 *										If false, this will only apply the package namespace if the text namespace already contains package namespace markers.
 *
 * @return The full namespace that should be used by the localization system.
 */
CORE_API FString BuildFullNamespace(const FString& InTextNamespace, const FString& InPackageNamespace, const bool bAlwaysApplyPackageNamespace = false);

/**
 * Given a text namespace, extract any package namespace that may currently be present.
 *
 * @param InTextNamespace				The namespace currently used by the FText instance.
 *
 * @return The extracted package namespace component, or an empty string if there was no package namespace component.
 */
CORE_API FString ExtractPackageNamespace(const FString& InTextNamespace);

/**
 * Given a text namespace, strip any package namespace that may currently be present.
 * This is similar to calling BuildFullNamespace with an empty package namespace, however this version will also remove the package namespace markers.
 *
 * @param InTextNamespace				The namespace currently used by the FText instance.
 *
 * @return The namespace stripped of any package namespace component.
 */
CORE_API FString StripPackageNamespace(const FString& InTextNamespace);
CORE_API void StripPackageNamespaceInline(FString& InOutTextNamespace);

enum class ETextCopyMethod : uint8
{
	/** Give the text a new key if the full namespace changes */
	NewKey,
	/** Keep the existing key if the full namespace changes */
	PreserveKey,
	/** Copy the text verbatim, disregarding any full namespace changes */
	Verbatim,
};

/**
 * Make a copy of the given text that's valid to use with the given package namespace, optionally preserving its existing key.
 * @note Returns the result verbatim if the given package namespace is empty, or if there is no change when applying the package namespace to the text.
 *
 * @param InText						The current FText instance.
 * @param InPackageNamespace			The namespace of the destination package of the FText instance.
 * @param InCopyMethod					The method that should be used to copy the FText instance.
 * @param bAlwaysApplyPackageNamespace	If true, this will always apply the package namespace to the text namespace (always treated as ETextCopyMethod::Verbatim when USE_STABLE_LOCALIZATION_KEYS is false).
 *										If false, this will only apply the package namespace if the text namespace already contains package namespace markers.
 *
 * @return A copy of the given text that's valid to use with the given package namespace.
 */
CORE_API FText CopyTextToPackage(const FText& InText, const FString& InPackageNamespace, const ETextCopyMethod InCopyMethod = ETextCopyMethod::NewKey, const bool bAlwaysApplyPackageNamespace = false);

#if USE_STABLE_LOCALIZATION_KEYS

/**
 * Given an archive, try and get the package namespace it should use for localization.
 *
 * @param InArchive						The archive to try and get the package namespace for.
 *
 * @return The package namespace, or an empty string if the archive has no package namespace set.
 */
CORE_API FString GetPackageNamespace(FArchive& InArchive);

#endif // USE_STABLE_LOCALIZATION_KEYS

}
