// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"

/**
 * Utility for performing low-level localized transforms.
 * The implementation can be found in LegacyText.cpp and ICUText.cpp.
 */
class FTextTransformer
{
public:
	static CORE_API FString ToLower(const FString& InStr);
	static CORE_API FString ToUpper(const FString& InStr);
};
