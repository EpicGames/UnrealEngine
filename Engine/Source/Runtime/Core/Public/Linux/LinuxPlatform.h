// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Not included directly

// Base defines, must define these for the platform, there are no defaults
#ifndef PLATFORM_DESKTOP
	#define PLATFORM_DESKTOP					1
#endif

#include "Unix/UnixPlatform.h"

#define PLATFORM_GLOBAL_LOG_CATEGORY			LogLinux

#define PLATFORM_SUPPORTS_BORDERLESS_WINDOW		1
