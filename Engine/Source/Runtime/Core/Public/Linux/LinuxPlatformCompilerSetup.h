// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	LinuxPlatformCompilerSetup.h: pragmas, version checks and other things for the Linux compiler
==============================================================================================*/

#pragma once

#include "Unix/UnixPlatformCompilerPreSetup.h"
