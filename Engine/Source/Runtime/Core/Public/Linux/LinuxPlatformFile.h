// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	LinuxPlatformFile.h: Linux platform File functions
==============================================================================================*/

#pragma once

#include "Unix/UnixPlatformFile.h" // IWYU pragma: export
