// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	LinuxPlatformMisc.h: Linux platform misc functions
==============================================================================================*/

#pragma once

#include "Unix/UnixPlatformMisc.h"

/**
 * Linux implementation of the misc OS functions
 */

struct FLinuxPlatformMisc : public FUnixPlatformMisc
{
	static CORE_API void PlatformInit();
	static CORE_API void PlatformTearDown();

	/**
	 * Returns the current network connection type by checking non-loopback interfaces via getifaddrs().
	 */
	static CORE_API ENetworkConnectionType GetNetworkConnectionType();

	/**
	 * Checks the netlink socket for pending network change events and updates
	 * connection status if a change occurred. Should be called from PumpMessages().
	 */
	static CORE_API void CheckNetworkConnectionEvents();

private:
	/** Creates a NETLINK_ROUTE socket to listen for interface and address changes. */
	static void StartNetworkConnectionMonitoring();

	/** Closes the netlink socket. */
	static void StopNetworkConnectionMonitoring();
};

typedef FLinuxPlatformMisc FPlatformMisc;
