// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	LinuxPlatformProcess.h: Linux platform Process functions
==============================================================================================*/

#pragma once

#include "HAL/Platform.h"
#include "Unix/UnixPlatformProcess.h" // IWYU pragma: export

/**
 * Linux implementation of the Process OS functions
 */
struct FLinuxPlatformProcess : public FUnixPlatformProcess
{
	static CORE_API const TCHAR* BaseDir();
	static CORE_API const TCHAR* GetBinariesSubdirectory();
	static CORE_API EWaitAndForkResult WaitAndFork();
};

typedef FLinuxPlatformProcess FPlatformProcess;
