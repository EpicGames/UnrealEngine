// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	LinuxPlatformProperties.h - Basic static properties of a platform 
	These are shared between:
		the runtime platform - via FPlatformProperties
		the target platforms - via ITargetPlatform
==================================================================================*/

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformProperties.h"


/**
 * Implements Linux platform properties.
 */
template<bool HAS_EDITOR_DATA, bool IS_DEDICATED_SERVER, bool IS_CLIENT_ONLY, bool IS_ARM64>
struct FLinuxPlatformProperties
	: public FGenericPlatformProperties
{
	static constexpr UE_FORCEINLINE_HINT bool HasEditorOnlyData()
	{
		return HAS_EDITOR_DATA;
	}

	static constexpr UE_FORCEINLINE_HINT const char* IniPlatformName()
	{
		return IS_ARM64 ? "LinuxArm64" : "Linux";
	}

	static constexpr UE_FORCEINLINE_HINT const TCHAR* GetRuntimeSettingsClassName()
	{
		return TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings");
	}

	static constexpr UE_FORCEINLINE_HINT bool IsGameOnly()
	{
		return UE_GAME;
	}

	static constexpr UE_FORCEINLINE_HINT bool IsServerOnly()
	{
		return IS_DEDICATED_SERVER;
	}

	static constexpr UE_FORCEINLINE_HINT bool IsClientOnly()
	{
		return IS_CLIENT_ONLY;
	}

	static constexpr UE_FORCEINLINE_HINT bool IsArm64()
	{
		return IS_ARM64;
	}

	static constexpr inline const char* PlatformName()
	{
		if (IS_DEDICATED_SERVER)
		{
			return IS_ARM64 ? "LinuxArm64Server" : "LinuxServer";
		}

		if (HAS_EDITOR_DATA)
		{
			return "LinuxEditor";
		}

		if (IS_CLIENT_ONLY)
		{
			return IS_ARM64 ? "LinuxArm64Client" : "LinuxClient";
		}

		return IS_ARM64 ? "LinuxArm64" : "Linux";
	}

	static constexpr UE_FORCEINLINE_HINT bool RequiresCookedData()
	{
		return !HAS_EDITOR_DATA;
	}

	static constexpr UE_FORCEINLINE_HINT bool HasSecurePackageFormat()
	{
		return IS_DEDICATED_SERVER;
	}

	static constexpr UE_FORCEINLINE_HINT bool RequiresUserCredentials()
	{
		return true;
	}

	static constexpr inline bool SupportsAutoSDK()
	{
// linux cross-compiling / cross-building from windows supports AutoSDK.  But hosted linux doesn't yet.
#if PLATFORM_WINDOWS
		return true;
#else
		return false;
#endif
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMultipleGameInstances()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool HasFixedResolution()
	{
		return false;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsWindowedMode()
	{
		return !IS_DEDICATED_SERVER;
	}

	static constexpr UE_FORCEINLINE_HINT bool AllowsFramerateSmoothing()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsRayTracing()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsQuit()
	{
		return true;
	}

	static constexpr inline float GetVariantPriority()
	{
		if (IS_DEDICATED_SERVER)
		{
			return 0.0f;
		}

		if (HAS_EDITOR_DATA)
		{
			return 0.0f;
		}

		if (IS_CLIENT_ONLY)
		{
			return 0.0f;
		}

		return 1.0f;
	}

	static constexpr UE_FORCEINLINE_HINT bool AllowsCallStackDumpDuringAssert()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsAudioStreaming()
	{
		return !IsServerOnly();
	}

	static constexpr UE_FORCEINLINE_HINT int64 GetMemoryMappingAlignment()
	{
		return 4096;
	}
};

#ifdef PROPERTY_HEADER_SHOULD_DEFINE_TYPE
typedef FLinuxPlatformProperties<WITH_EDITORONLY_DATA, UE_SERVER, !WITH_SERVER_CODE && !WITH_EDITOR, !!PLATFORM_CPU_ARM_FAMILY> FPlatformProperties;
#endif
