// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Logging/TokenizedMessage.h"

/**
 * A message log.
 * Message logs can be written to from any module, incorporating rich tokenized information as well
 * as text. Messages will be displayed to the standard log and the rich MessageLogListing if it is available.
 */
class IMessageLog
{
public:

	/**
	 * Virtual destructor
	 */
	virtual ~IMessageLog() = default;

	/** 
	 * Appends a single message
	 * 
	 * @param	NewMessage	The message to append
	 * @param	bMirrorToOutputLog	Whether or not the message should also be mirrored to the output log
	 */
	virtual void AddMessage( const TSharedRef<class FTokenizedMessage>& NewMessage, bool bMirrorToOutputLog = true ) = 0;

	/** 
	 * Appends multiple messages
	 * 
	 * @param	NewMessages	The messages to append.
	 * @param	bMirrorToOutputLog	Whether or not the messages should also be mirrored to the output log
	 */
	virtual void AddMessages( const TArray< TSharedRef<class FTokenizedMessage> >& NewMessages, bool bMirrorToOutputLog = true ) = 0;

	/** 
	 * Adds a new page to the log.
	 * Old pages are only kept around if they contain messages, so if the current
	 * page is empty, this call does nothing.
	 * @param	Title	the title of the new page
	 */
	virtual void NewPage( const FText& Title ) = 0;

	/** 
	* Sets the current page to the one specified by the title.
	* @param	Title	the title of the page to switch to
	*/
	virtual void SetCurrentPage( const FText& Title ) = 0;

	/** 
	* Sets the current page to the one specified by the title.
	* @param	InOldPageIndex	the index of the page to switch to. This page will move to index 0.
	*/
	virtual void SetCurrentPage( const uint32 InOldPageIndex ) = 0;

	/** 
	 * Notify the user if there are any messages on the current page for this log.
	 * If there are no messages present, this call does nothing.
	 * @param	Message			The message to display in the notification.
	 * @param	SeverityFilter	Notifications will only be displayed if there are messages present
	 *							that are of equal or greater severity than this.
	 * @param	bForce			Notify anyway, even if the filters gives us no messages.
	 */
	virtual void NotifyIfAnyMessages( const FText& Message, EMessageSeverity::Type SeverityFilter = EMessageSeverity::Info, bool bForce = false ) = 0;

	/** 
	 * Opens up the message log to this listing.
	 */
	virtual void Open() = 0;

	/** 
	 * Checks to see if there are any messages according to the passed-in severity
	 *
	 * @param	SeverityFilter	Function will only return true if all the messages
	 *							are of equal or greater severity than this.
	 * @return The number of messages that pass our filter
	 */
	virtual int32 NumMessages( EMessageSeverity::Type SeverityFilter ) = 0;
};
