// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogVerbosity.h"
#include "Misc/Build.h"
#include "UObject/NameTypes.h"

#define UE_API CORE_API

#if defined(UE_LOG_LAZY_CATEGORY_NAMES) && UE_LOG_LAZY_CATEGORY_NAMES
using FLogCategoryName = FLazyName;
#define UE_FNAME_TO_LOG_CATEGORY_NAME(Name) FLazyName(Name)
#else
using FLogCategoryName = FName;
#define UE_FNAME_TO_LOG_CATEGORY_NAME(Name) Name
#endif

/** Base class for all log categories. */
struct FLogCategoryBase
{
	/**
	 * Constructor, registers with the log suppression system and sets up the default values.
	 *
	 * @param CategoryName           Name of the category.
	 * @param DefaultVerbosity       Default verbosity used to filter this category at runtime.
	 * @param CompileTimeVerbosity   Verbosity used to filter this category at compile time.
	 */
	UE_API FLogCategoryBase(const FLogCategoryName& CategoryName, ELogVerbosity::Type DefaultVerbosity, ELogVerbosity::Type CompileTimeVerbosity);

	/**
	 * Constructor, registers with the log suppression system and sets up the default values.
	 *
	 * @param CategoryName           Name of the category.
	 * @param DefaultVerbosity       Default verbosity used to filter this category at runtime.
	 * @param CompileTimeVerbosity   Verbosity used to filter this category at compile time.
	 */
	UE_API FLogCategoryBase(const TCHAR* CategoryName, ELogVerbosity::Type DefaultVerbosity, ELogVerbosity::Type CompileTimeVerbosity);

	/** Destructor, unregisters from the log suppression system. */
	UE_API ~FLogCategoryBase();

	/** Should not generally be used directly. Tests the runtime verbosity and maybe triggers a debug break, etc. */
	UE_FORCEINLINE_HINT bool IsSuppressed(ELogVerbosity::Type VerbosityLevel) const
	{
		return !((VerbosityLevel & ELogVerbosity::VerbosityMask) <= Verbosity);
	}

	/** Called just after a logging statement being allow to print. Checks a few things and maybe breaks into the debugger. */
	UE_API void PostTrigger(ELogVerbosity::Type VerbosityLevel);

	inline const FLogCategoryName& GetCategoryName() const { return CategoryName; }

	/** Gets the working verbosity. */
	inline ELogVerbosity::Type GetVerbosity() const { return (ELogVerbosity::Type)Verbosity; }
	
	/** Sets up the working verbosity and clamps to the compile time verbosity. */
	UE_API void SetVerbosity(ELogVerbosity::Type Verbosity);

	/** Gets the compile time verbosity. */
	inline ELogVerbosity::Type GetCompileTimeVerbosity() const { return CompileTimeVerbosity; }

private:
	friend class FLogSuppressionImplementation;
	friend class FLogScopedVerbosityOverride;

	/** Internal call to set up the working verbosity from the boot time default. */
	void ResetFromDefault();

	/** Holds the current suppression state */
	ELogVerbosity::Type Verbosity;
	/** Holds the break flag */
	bool DebugBreakOnLog;
	/** Holds default suppression */
	uint8 DefaultVerbosity;
	/** Holds compile time suppression */
	const ELogVerbosity::Type CompileTimeVerbosity;
	/** Name for this category */

	const FLogCategoryName CategoryName;
};

/** Template for log categories that transfers the compile-time constant default and compile time verbosity to the FLogCategoryBase constructor. */
template <ELogVerbosity::Type InDefaultVerbosity, ELogVerbosity::Type InCompileTimeVerbosity>
struct FLogCategory : public FLogCategoryBase
{
	static_assert((InDefaultVerbosity & ELogVerbosity::VerbosityMask) < ELogVerbosity::NumVerbosity, "Bogus default verbosity.");
	static_assert(InCompileTimeVerbosity < ELogVerbosity::NumVerbosity, "Bogus compile time verbosity.");

	static constexpr ELogVerbosity::Type CompileTimeVerbosity = InCompileTimeVerbosity;

	inline constexpr ELogVerbosity::Type GetCompileTimeVerbosity() const { return CompileTimeVerbosity; }

	UE_FORCEINLINE_HINT FLogCategory(const FLogCategoryName& InCategoryName)
		: FLogCategoryBase(InCategoryName, InDefaultVerbosity, CompileTimeVerbosity)
	{
	}

	UE_FORCEINLINE_HINT FLogCategory(const TCHAR* InCategoryName)
		: FLogCategoryBase(InCategoryName, InDefaultVerbosity, CompileTimeVerbosity)
	{
	}
};

#if NO_LOGGING

	struct FNoLoggingCategory {};

	#define DECLARE_LOG_CATEGORY_EXTERN(CategoryName, DefaultVerbosity, CompileTimeVerbosity) extern FNoLoggingCategory CategoryName;
	#define DEFINE_LOG_CATEGORY(CategoryName, ...) FNoLoggingCategory CategoryName;
	#define DEFINE_LOG_CATEGORY_STATIC(...)
	#define DECLARE_LOG_CATEGORY_CLASS(...)
	#define DEFINE_LOG_CATEGORY_CLASS(...)

#else

	#define UE_PRIVATE_DEFINE_LOG_CATEGORY(CategoryName, DefaultVerbosity, CompileTimeVerbosity) \
		struct FLogCategory##CategoryName : public ::FLogCategory<::ELogVerbosity::DefaultVerbosity, ::ELogVerbosity::CompileTimeVerbosity> \
		{ \
			static_assert(::ELogVerbosity::DefaultVerbosity <= ::ELogVerbosity::CompileTimeVerbosity, \
				"Default verbosity of log category " #CategoryName " cannot be higher than its compile time verbosity."); \
			UE_FORCEINLINE_HINT FLogCategory##CategoryName() : FLogCategory(TEXT(#CategoryName)) {} \
		} CategoryName;

	/**
	 * A macro to declare a logging category as a C++ "extern", usually declared in the header and paired with DEFINE_LOG_CATEGORY in the source. Accessible by all files that include the header.
	 *
	 * @param CategoryName   Name of the category, typically prefixed with Log.
	 * @param DefaultVerbosity   Default runtime verbosity of the category.
	 * @param CompileTimeVerbosity   Maximum verbosity to compile into the code.
	 */
	#define DECLARE_LOG_CATEGORY_EXTERN(CategoryName, DefaultVerbosity, CompileTimeVerbosity) \
		extern UE_PRIVATE_DEFINE_LOG_CATEGORY(CategoryName, DefaultVerbosity, CompileTimeVerbosity);

	/**
	 * A macro to define a logging category, usually paired with DECLARE_LOG_CATEGORY_EXTERN from the header.
	 * @param CategoryName category to define
	 */
	#define DEFINE_LOG_CATEGORY(CategoryName) FLogCategory##CategoryName CategoryName;

	/**
	 * A macro to define a logging category as a C++ "static". This should ONLY be declared in a source file. Only accessible in that single file.
	 *
	 * @param CategoryName   Name of the category, typically prefixed with Log.
	 * @param DefaultVerbosity   Default runtime verbosity of the category.
	 * @param CompileTimeVerbosity   Maximum verbosity to compile into the code.
	 */
	//-V:DEFINE_LOG_CATEGORY_STATIC:501
	#define DEFINE_LOG_CATEGORY_STATIC(CategoryName, DefaultVerbosity, CompileTimeVerbosity) \
		static UE_PRIVATE_DEFINE_LOG_CATEGORY(CategoryName, DefaultVerbosity, CompileTimeVerbosity);

	/**
	 * A macro to declare a logging category as a C++ "class static"
	 *
	 * @param CategoryName   Name of the category, typically prefixed with Log.
	 * @param DefaultVerbosity   Default runtime verbosity of the category.
	 * @param CompileTimeVerbosity   Maximum verbosity to compile into the code.
	 */
	//-V:DECLARE_LOG_CATEGORY_CLASS:501
	#define DECLARE_LOG_CATEGORY_CLASS(CategoryName, DefaultVerbosity, CompileTimeVerbosity) \
		static UE_PRIVATE_DEFINE_LOG_CATEGORY(CategoryName, DefaultVerbosity, CompileTimeVerbosity)

	/**
	 * A macro to define a logging category, usually paired with DECLARE_LOG_CATEGORY_CLASS from the header.
	 * @param CategoryName category to define
	 */
	#define DEFINE_LOG_CATEGORY_CLASS(Class, CategoryName) Class::FLogCategory##CategoryName Class::CategoryName;

#endif // NO_LOGGING

#undef UE_API
