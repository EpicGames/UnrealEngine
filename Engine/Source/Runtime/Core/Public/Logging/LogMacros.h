// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/CharType.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PreprocessorHelpers.h"
#include "Logging/LogCategory.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "Logging/LogTrace.h"
#include "Logging/LogVerbosity.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Build.h"
#include "Misc/VarArgs.h"
#include "String/FormatStringSan.h"
#include "Templates/EnableIf.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Templates/IsValidVariadicFunctionArg.h"
#include "Traits/ElementType.h"
#include "Traits/IsCharEncodingCompatibleWith.h"

#include <type_traits>

#ifndef UE_LOG_INCLUDE_SOURCE_LOCATION
#define UE_LOG_INCLUDE_SOURCE_LOCATION 1
#endif

#if UE_LOG_INCLUDE_SOURCE_LOCATION || LOGTRACE_ENABLED
#define UE_PRIVATE_LOG_USE_STATIC_RECORD 1
#else
#define UE_PRIVATE_LOG_USE_STATIC_RECORD 0
#endif

/*----------------------------------------------------------------------------
	Logging
----------------------------------------------------------------------------*/

// Define NO_LOGGING to strip out all writing to log files, OutputDebugString(), etc.

/**
 * FMsg 
 * This struct contains functions for messaging with tools or debug logs.
 **/
struct FMsg
{
	/** Sends a message to a remote tool. */
	CORE_API static void SendNotificationString( const TCHAR* Message );

	/** Sends a formatted message to a remote tool. */
	template <typename FmtType, typename... Types>
	static void SendNotificationStringf(const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a const TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FMsg::SendNotificationStringf");

		SendNotificationStringfImpl((const TCHAR*)Fmt, Args...);
	}

	/** Log function */
	template <typename FmtType, typename... Types>
	static void Logf(const ANSICHAR* File, int32 Line, const FLogCategoryName& Category, ELogVerbosity::Type Verbosity, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a const TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FMsg::Logf");

		LogfImpl(File, Line, Category, Verbosity, (const TCHAR*)Fmt, Args...);
	}

	/** Internal version of log function. Should be used only in logging macros, as it relies on caller to call assert on fatal error */
	template <typename FmtType, typename... Types>
	static void Logf_Internal(const ANSICHAR* File, int32 Line, const FLogCategoryName& Category, ELogVerbosity::Type Verbosity, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a const TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FMsg::Logf_Internal");

		Logf_InternalImpl(File, Line, Category, Verbosity, (const TCHAR*)Fmt, Args...);
	}

	CORE_API static void LogV(const ANSICHAR* File, int32 Line, const FLogCategoryName& Category, ELogVerbosity::Type Verbosity, const TCHAR* Fmt, va_list Args);

private:
	CORE_API static void VARARGS LogfImplV(const ANSICHAR* File, int32 Line, const FLogCategoryName& Category, ELogVerbosity::Type Verbosity, void* ReturnAddress, const TCHAR* Fmt, va_list Args);
	CORE_API static void VARARGS LogfImpl(const ANSICHAR* File, int32 Line, const FLogCategoryName& Category, ELogVerbosity::Type Verbosity, const TCHAR* Fmt, ...);
	CORE_API static void VARARGS Logf_InternalImpl(const ANSICHAR* File, int32 Line, const FLogCategoryName& Category, ELogVerbosity::Type Verbosity, const TCHAR* Fmt, ...);
	CORE_API static void VARARGS SendNotificationStringfImpl(const TCHAR* Fmt, ...);
};

/*----------------------------------------------------------------------------
	Logging suppression
----------------------------------------------------------------------------*/

#ifndef COMPILED_IN_MINIMUM_VERBOSITY
	#define COMPILED_IN_MINIMUM_VERBOSITY VeryVerbose
#else
	#if !IS_MONOLITHIC
		#error COMPILED_IN_MINIMUM_VERBOSITY can only be defined in monolithic builds.
	#endif
#endif

#define UE_LOG_EXPAND_IS_FATAL(Verbosity, ActiveBlock, InactiveBlock) UE_JOIN(UE_LOG_EXPAND_IS_FATAL_, Verbosity)(ActiveBlock, InactiveBlock)

#define UE_LOG_EXPAND_IS_FATAL_Fatal(      ActiveBlock, InactiveBlock) ActiveBlock
#define UE_LOG_EXPAND_IS_FATAL_Error(      ActiveBlock, InactiveBlock) InactiveBlock
#define UE_LOG_EXPAND_IS_FATAL_Warning(    ActiveBlock, InactiveBlock) InactiveBlock
#define UE_LOG_EXPAND_IS_FATAL_Display(    ActiveBlock, InactiveBlock) InactiveBlock
#define UE_LOG_EXPAND_IS_FATAL_Log(        ActiveBlock, InactiveBlock) InactiveBlock
#define UE_LOG_EXPAND_IS_FATAL_Verbose(    ActiveBlock, InactiveBlock) InactiveBlock
#define UE_LOG_EXPAND_IS_FATAL_VeryVerbose(ActiveBlock, InactiveBlock) InactiveBlock
#define UE_LOG_EXPAND_IS_FATAL_All(        ActiveBlock, InactiveBlock) InactiveBlock
#define UE_LOG_EXPAND_IS_FATAL_SetColor(   ActiveBlock, InactiveBlock) InactiveBlock

#if DO_CHECK
	#define UE_LOG_SOURCE_FILE(File) File
#else
	#define UE_LOG_SOURCE_FILE(File) "Unknown"
#endif

namespace UE::Logging::Private
{

/** Data about a static basic log that is created on-demand. */
struct FStaticBasicLogDynamicData
{
#if LOGTRACE_ENABLED
	std::atomic<bool> bInitializedTrace = false;
#endif
};

/** Data about a static basic log that is constant for every occurrence. */
template <UE::CCharType FormatCharType>
struct TStaticBasicLogRecord
{
	const FormatCharType* Format = nullptr;
	UE_IF(UE_LOG_INCLUDE_SOURCE_LOCATION,, inline static constexpr) const ANSICHAR* File = nullptr;
	UE_IF(UE_LOG_INCLUDE_SOURCE_LOCATION,, inline static constexpr) int32 Line = 0;
#if LOGTRACE_ENABLED
	FStaticBasicLogDynamicData& DynamicData;
#endif

	// Workaround for https://developercommunity.visualstudio.com/t/Incorrect-warning-C4700-with-unrelated-s/10285950
	constexpr TStaticBasicLogRecord(
		const FormatCharType* InFormat,
		const ANSICHAR* InFile,
		int32 InLine,
		FStaticBasicLogDynamicData* InDynamicData)
		: Format(InFormat)
	#if UE_LOG_INCLUDE_SOURCE_LOCATION
		, File(InFile)
		, Line(InLine)
	#endif
	#if LOGTRACE_ENABLED
		, DynamicData(*InDynamicData)
	#endif
	{
	}

#if !UE_PRIVATE_LOG_USE_STATIC_RECORD
	// Delete the address-of operator because it is unsafe to pass pointers to records
	// when they do not have static storage duration.
	TStaticBasicLogRecord* operator&() const = delete;
#endif
};

// Pass TStaticBasicLogRecord by value when the record is not static, because in that
// configuration it is simply a wrapper for the format string pointer.
#if UE_PRIVATE_LOG_USE_STATIC_RECORD
template <UE::CCharType FormatCharType>
using TStaticBasicLogRecordParam = const TStaticBasicLogRecord<FormatCharType>&;
#else
template <UE::CCharType FormatCharType>
using TStaticBasicLogRecordParam = TStaticBasicLogRecord<FormatCharType>;
#endif

template <ELogVerbosity::Type Verbosity, UE::CCharType FormatCharType>
CORE_API void BasicLog(TStaticBasicLogRecordParam<FormatCharType> Log, const FLogCategoryBase* Category, ...);
template <UE::CCharType FormatCharType>
CORE_API void BasicFatalLog(TStaticBasicLogRecordParam<FormatCharType> Log, const FLogCategoryBase* Category, ...);
template <UE::CCharType FormatCharType>
CORE_API void BasicFatalLogWithProgramCounter(TStaticBasicLogRecordParam<FormatCharType> Log, const FLogCategoryBase* Category, void* ProgramCounter, ...);

} // UE::Logging::Private

#if NO_LOGGING

	// This will only log Fatal errors
	#define UE_LOG(CategoryName, Verbosity, Format, ...) \
	{ \
		if constexpr ((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) == ::ELogVerbosity::Fatal) \
		{ \
			LowLevelFatalError(Format, ##__VA_ARGS__); \
			CA_ASSUME(false); \
		} \
	}
	// DO NOT USE. Use UE_LOG because this will be deprecated in a future release.
	#define UE_LOG_REF UE_LOG
	// DO NOT USE. Use UE_LOG because this will be deprecated in a future release.
	#define UE_LOG_CLINKAGE UE_LOG
	#define UE_LOGF(CategoryName, Verbosity, Format, ...) \
	{ \
		if constexpr ((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) == ::ELogVerbosity::Fatal) \
		{ \
			LowLevelFatalError(Format, ##__VA_ARGS__); \
			CA_ASSUME(false); \
		} \
	}

	// Conditional logging (fatal errors only).
	#define UE_CLOG(Condition, CategoryName, Verbosity, Format, ...) \
	{ \
		if constexpr ((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) == ::ELogVerbosity::Fatal) \
		{ \
			if (Condition) \
			{ \
				LowLevelFatalError(Format, ##__VA_ARGS__); \
				CA_ASSUME(false); \
			} \
		} \
	}
	#define UE_CLOGF(Condition, CategoryName, Verbosity, Format, ...) \
	{ \
		if constexpr ((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) == ::ELogVerbosity::Fatal) \
		{ \
			if (Condition) \
			{ \
				LowLevelFatalError(Format, ##__VA_ARGS__); \
				CA_ASSUME(false); \
			} \
		} \
	}

	#define UE_LOG_ACTIVE(...)				0
	#define UE_LOG_ANY_ACTIVE(...)			0
	#define UE_SUPPRESS(...) {}
	#define UE_GET_LOG_VERBOSITY(...)		(::ELogVerbosity::NoLogging)
	#define UE_SET_LOG_VERBOSITY(...)

#else

	namespace UEAsserts_Private
	{
		template <int32 VerbosityToCheck, typename CategoryType>
		UE_FORCEINLINE_HINT bool IsLogActive(const CategoryType& Category)
		{
			if constexpr (((VerbosityToCheck & ELogVerbosity::VerbosityMask) <= CategoryType::CompileTimeVerbosity &&
				(VerbosityToCheck & ELogVerbosity::VerbosityMask) <= ELogVerbosity::COMPILED_IN_MINIMUM_VERBOSITY))
			{
				return !Category.IsSuppressed((ELogVerbosity::Type)VerbosityToCheck);
			}
			else
			{
				return false;
			}
		}
	}

	/** 
	 * A predicate that returns true if the given logging category is active (logging) at a given verbosity level 
	 * @param CategoryName name of the logging category
	 * @param Verbosity, verbosity level to test against
	**/
	#define UE_LOG_ACTIVE(CategoryName, Verbosity) (::UEAsserts_Private::IsLogActive<(int32)::ELogVerbosity::Verbosity>(CategoryName))

	#define UE_GET_LOG_VERBOSITY(CategoryName) \
		CategoryName.GetVerbosity()

	#define UE_SET_LOG_VERBOSITY(CategoryName, Verbosity) \
		CategoryName.SetVerbosity(::ELogVerbosity::Verbosity);

	/**
	 * A macro that logs a formatted message if the log category is active at the requested verbosity level.
	 * Prefer UE_LOGF because its ASCII or UTF-8 format strings require less memory. 
	 *
	 * @param CategoryName   Name of the log category as provided to DEFINE_LOG_CATEGORY.
	 * @param Verbosity      Verbosity level of this message. See ELogVerbosity.
	 * @param Format         Format string literal in the style of printf. Specify as TEXT("...")
	 */
	#define UE_LOG(CategoryName, Verbosity, Format, ...) \
	{ \
		static_assert(::TIsArrayOrRefOfTypeByPredicate<decltype(Format), ::TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array."); \
		UE_PRIVATE_LOG(UE_EMPTY, constexpr, CategoryName, Verbosity, Format, ##__VA_ARGS__) \
	}

	/**
	 * A macro that logs a formatted message if the log category is active at the requested verbosity level.
	 *
	 * @param CategoryName   Name of the log category as provided to DEFINE_LOG_CATEGORY.
	 * @param Verbosity      Verbosity level of this message. See ELogVerbosity.
	 * @param Format         Format string literal in the style of printf. Specify as "..." or UTF8TEXT("...")
	 */
	#define UE_LOGF(CategoryName, Verbosity, Format, ...) \
	{ \
		static_assert(::TIsArrayOrRefOfTypeByPredicate<decltype(Format), ::TIsCharEncodingCompatibleWithUTF8CHAR>::Value, "Formatting string must be a UTF8CHAR or ANSICHAR array."); \
		UE_PRIVATE_LOG(UE_EMPTY, constexpr, CategoryName, Verbosity, Format, ##__VA_ARGS__) \
	}

	/**
	 * DO NOT USE. A macro that logs a formatted message if the log category is active at the requested verbosity level.
	 *
	 * @note This does not trace the category correctly and will be deprecated in a future release.
	 *
	 * @param CategoryRef   A reference to an instance of FLogCategoryBase.
	 * @param Verbosity     Verbosity level of this message. See ELogVerbosity.
	 * @param Format        Format string literal in the style of printf.
	 */
	#define UE_LOG_REF(CategoryRef, Verbosity, Format, ...) \
	{ \
		static_assert(::TIsArrayOrRefOfTypeByPredicate<decltype(Format), ::TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array."); \
		UE_PRIVATE_LOG(UE_EMPTY, UE_EMPTY, CategoryRef, Verbosity, Format, ##__VA_ARGS__) \
	}

	// DO NOT USE. Use UE_LOG because this will be deprecated in a future release.
	#define UE_LOG_CLINKAGE UE_LOG

	/**
	 * A macro that conditionally logs a formatted message if the log category is active at the requested verbosity level.
	 *
	 * @note The condition is not evaluated unless the log category is active at the requested verbosity level.
	 *
	 * @param Condition      Condition that must evaluate to true in order for the message to be logged.
	 * @param CategoryName   Name of the log category as provided to DEFINE_LOG_CATEGORY.
	 * @param Verbosity      Verbosity level of this message. See ELogVerbosity.
	 * @param Format         Format string literal in the style of printf.
	 */
	#define UE_CLOG(Condition, CategoryName, Verbosity, Format, ...) \
	{ \
		static_assert(::TIsArrayOrRefOfTypeByPredicate<decltype(Format), ::TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array."); \
		UE_PRIVATE_LOG(if (Condition), constexpr, CategoryName, Verbosity, Format, ##__VA_ARGS__) \
	}
	#define UE_CLOGF(Condition, CategoryName, Verbosity, Format, ...) \
	{ \
		static_assert(::TIsArrayOrRefOfTypeByPredicate<decltype(Format), ::TIsCharEncodingCompatibleWithUTF8CHAR>::Value, "Formatting string must be a UTF8CHAR or ANSICHAR array."); \
		UE_PRIVATE_LOG(if (Condition), constexpr, CategoryName, Verbosity, UTF8TEXT(Format), ##__VA_ARGS__) \
	}

	/** Private macro used to implement the public log macros. DO NOT CALL DIRECTLY! */
	#define UE_PRIVATE_LOG(Condition, CategoryConst, Category, Verbosity, Format, ...) \
	{ \
		UE_VALIDATE_FORMAT_STRING(Format, ##__VA_ARGS__); \
		UE_IF(LOGTRACE_ENABLED, static ::UE::Logging::Private::FStaticBasicLogDynamicData LOG_Dynamic, ); \
		UE_IF(UE_PRIVATE_LOG_USE_STATIC_RECORD, static,) constexpr ::UE::Logging::Private::TStaticBasicLogRecord<TElementType_T<decltype(Format)>> LOG_Static(Format, __builtin_FILE(), __builtin_LINE(), UE_IF(LOGTRACE_ENABLED, &LOG_Dynamic, nullptr)); \
		static_assert((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) < ::ELogVerbosity::NumVerbosity && ::ELogVerbosity::Verbosity > 0, "Verbosity must be constant and in range."); \
		if constexpr ((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) == ::ELogVerbosity::Fatal) \
		{ \
			Condition \
			{ \
				::UE::Logging::Private::BasicFatalLog(LOG_Static, &Category, ##__VA_ARGS__); \
				CA_ASSUME(false); \
			} \
		} \
		else if constexpr ((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) <= ::ELogVerbosity::COMPILED_IN_MINIMUM_VERBOSITY) \
		{ \
			if CategoryConst ((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) <= Category.GetCompileTimeVerbosity()) \
			{ \
				if (!Category.IsSuppressed(::ELogVerbosity::Verbosity)) \
				{ \
					Condition \
					{ \
						::UE::Logging::Private::BasicLog<::ELogVerbosity::Verbosity>(LOG_Static, &Category, ##__VA_ARGS__); \
					} \
				} \
			} \
		} \
	}

	/** 
	 * A macro that executes some code within a scope if a given logging category is active at a given verbosity level
	 * Also, withing the scope of the execution, the default category and verbosity is set up for the low level logging 
	 * functions.
	 * @param CategoryName name of the logging category
	 * @param Verbosity, verbosity level to test against
	 * @param ExecuteIfUnsuppressed, code to execute if the verbosity level for this category is being displayed
	 ***/
	#define UE_SUPPRESS(CategoryName, Verbosity, ExecuteIfUnsuppressed) \
	{ \
		static_assert((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) < ::ELogVerbosity::NumVerbosity && ::ELogVerbosity::Verbosity > 0, "Verbosity must be constant and in range."); \
		CA_CONSTANT_IF((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) <= ::ELogVerbosity::COMPILED_IN_MINIMUM_VERBOSITY && (::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) <= FLogCategory##CategoryName::CompileTimeVerbosity) \
		{ \
			if (!CategoryName.IsSuppressed(::ELogVerbosity::Verbosity)) \
			{ \
				::FScopedCategoryAndVerbosityOverride TEMP__##CategoryName(CategoryName.GetCategoryName(), ::ELogVerbosity::Type(::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask)); \
				ExecuteIfUnsuppressed; \
				CategoryName.PostTrigger(::ELogVerbosity::Verbosity); \
			} \
		} \
	}

#endif // NO_LOGGING

#if UE_BUILD_SHIPPING
#define NOTIFY_CLIENT_OF_SECURITY_EVENT_IF_NOT_SHIPPING(NetConnection, SecurityPrint) ;
#else
#define NOTIFY_CLIENT_OF_SECURITY_EVENT_IF_NOT_SHIPPING(NetConnection, SecurityPrint) \
	FNetControlMessage<NMT_SecurityViolation>::Send(NetConnection, SecurityPrint); \
	NetConnection->FlushNet(true)
#endif

extern CORE_API int32 GEnsureOnNANDiagnostic;

// Macro to either log an error or ensure on a NaN error.
#if DO_CHECK && !USING_CODE_ANALYSIS
namespace UEAsserts_Private
{
	CORE_API void VARARGS InternalLogNANDiagnosticMessage(const TCHAR* FormattedMsg, ...); // UE_LOG(LogCore, Error, _FormatString_, ##__VA_ARGS__);
}
#define logOrEnsureNanError(_FormatString_, ...) \
	if (!GEnsureOnNANDiagnostic)\
	{\
		static bool OnceOnly = false;\
		if (!OnceOnly)\
		{\
			UEAsserts_Private::InternalLogNANDiagnosticMessage(_FormatString_, ##__VA_ARGS__); \
			OnceOnly = true;\
		}\
	}\
	else\
	{\
		ensureMsgf(!GEnsureOnNANDiagnostic, _FormatString_, ##__VA_ARGS__); \
	}
#else
#define logOrEnsureNanError(_FormatString_, ...)
#endif // DO_CHECK
