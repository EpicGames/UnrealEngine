// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogVerbosity.h"
#include "UObject/NameTypes.h"

/** 
 * Helper class that uses thread local storage to set up the default category and verbosity for the low level logging functions.
 * This is what allow a UE_LOGF(LogHAL, Log, "..."); within a UE_LOG statement to know what the category and verbosity is.
 * When one of these goes out of scope, it restores the previous values.
**/
class FLogScopedCategoryAndVerbosityOverride
{
public:
	/** STructure to aggregate a category and verbosity pair **/
	struct FOverride
	{
		ELogVerbosity::Type Verbosity;
		FName Category;
		FOverride()
			: Verbosity(ELogVerbosity::Log)
		{
		}
		FOverride(FName InCategory, ELogVerbosity::Type InVerbosity)
			: Verbosity(InVerbosity)
			, Category(InCategory)
		{
		}
		void operator=(const FOverride& Other)
		{
			Verbosity = Other.Verbosity;
			Category = Other.Category;
		}
	};
private:
	/** Backup of the category, verbosity pairs that was present when we were constructed **/
	FOverride Backup;
public:
	/** Back up the existing category and verbosity pair, then sets them.*/
	CORE_API FLogScopedCategoryAndVerbosityOverride(FName Category, ELogVerbosity::Type Verbosity);

	/** Restore the category and verbosity overrides to the previous value.*/
	CORE_API ~FLogScopedCategoryAndVerbosityOverride();

	/** Manages a TLS slot with the current overrides for category and verbosity. **/
	static CORE_API FOverride* GetTLSCurrent();
};

typedef FLogScopedCategoryAndVerbosityOverride FScopedCategoryAndVerbosityOverride;
