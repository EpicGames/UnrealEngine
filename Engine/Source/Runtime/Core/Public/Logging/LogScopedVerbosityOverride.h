// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogVerbosity.h"
#include "Misc/Build.h"

struct FLogCategoryBase;

/*-----------------------------------------------------------------------------
	FLogScopedVerbosityOverride
-----------------------------------------------------------------------------*/
/** 
 * Helper class that allows setting scoped verbosity for log category. 
 * Saved what was previous verbosity for the category, and recovers it when it goes out of scope. 
 * Use Macro LOG_SCOPE_VERBOSITY_OVERRIDE for this
 **/
class FLogScopedVerbosityOverride
{
private:
	/** Backup of the category, verbosity pairs that was present when we were constructed **/
	FLogCategoryBase*	SavedCategory;
	ELogVerbosity::Type SavedVerbosity;

public:
	/** Back up the existing verbosity for the category then sets new verbosity.*/
	CORE_API FLogScopedVerbosityOverride(FLogCategoryBase* Category, ELogVerbosity::Type Verbosity);

	/** Restore the verbosity overrides for the category to the previous value.*/
	CORE_API ~FLogScopedVerbosityOverride();

	// Disable accidental copies
	FLogScopedVerbosityOverride(const FLogScopedVerbosityOverride&) = delete;
	FLogScopedVerbosityOverride& operator=(const FLogScopedVerbosityOverride&) = delete;
};

/** 
 * A macro to override Verbosity of the Category within the scope
 * @param CategoryName, category to declare
 * @param ScopeVerbosity, verbosity to override
 **/
#if NO_LOGGING
	#define LOG_SCOPE_VERBOSITY_OVERRIDE(...)
#else
	#define LOG_SCOPE_VERBOSITY_OVERRIDE(CategoryName, ScopeVerbosity) FLogScopedVerbosityOverride LogCategory##CategoryName##Override(&CategoryName, ScopeVerbosity)
#endif
