// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/*-----------------------------------------------------------------------------
	Log Suppression
-----------------------------------------------------------------------------*/

/** Interface to the log suppression system **/
class FLogSuppressionInterface
{
public:
	/** Singleton, returns a reference the global log suppression implementation. **/
	static CORE_API FLogSuppressionInterface& Get();
	/** Returns null after TearDown() has been called */
	static CORE_API FLogSuppressionInterface* TryGet();
	static CORE_API void TearDown();

	/** Used by FLogCategoryBase to register itself with the global category table **/
	virtual void AssociateSuppress(struct FLogCategoryBase* Destination) = 0;
	/** Used by FLogCategoryBase to unregister itself from the global category table **/
	virtual void DisassociateSuppress(struct FLogCategoryBase* Destination) = 0;
	/** Called by appInit once the config files and commandline are set up. The log suppression system uses these to setup the boot time defaults. **/
	virtual void ProcessConfigAndCommandLine() = 0;
};
