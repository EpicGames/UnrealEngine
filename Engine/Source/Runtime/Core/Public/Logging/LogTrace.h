// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "HAL/PreprocessorHelpers.h"
#include "Logging/LogVerbosity.h"
#include "Misc/Build.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Trace/Config.h"
#include "Trace/Trace.h"
#include "Traits/IsCharEncodingCompatibleWith.h"

#if !defined(LOGTRACE_ENABLED)
#if UE_TRACE_ENABLED && !UE_BUILD_SHIPPING
#define LOGTRACE_ENABLED 1
#else
#define LOGTRACE_ENABLED 0
#endif
#endif

#if LOGTRACE_ENABLED
#include "ProfilingDebugging/FormatArgsTrace.h"

#if !defined(LOGTRACE_RUNTIME_FORMATTING_ENABLED)
#define LOGTRACE_RUNTIME_FORMATTING_ENABLED 1
#endif

UE_TRACE_CHANNEL_EXTERN(LogChannel, CORE_API)

struct FLogCategoryBase;
namespace UE { namespace Trace { class FChannel; } }

struct FLogTrace
{
	CORE_API static void OutputLogCategory(const FLogCategoryBase* Category, const TCHAR* Name, ELogVerbosity::Type DefaultVerbosity);
	CORE_API static void OutputLogMessageSpec(const void* LogPoint, const FLogCategoryBase* Category, ELogVerbosity::Type Verbosity, const ANSICHAR* File, int32 Line, const TCHAR* Format);
	CORE_API static void OutputLogMessageSimple(const void* LogPoint, const TCHAR* Fmt, ...);

	template <typename... Types>
	FORCENOINLINE static void OutputLogMessage(const void* LogPoint, Types... FormatArgs)
	{
		// This inline size covers 99.96% of log messages in sampled traces
		FFormatArgsTrace::TBufferWithOverflow<1600> Buffer;
		const int32 FormatArgsSize = FFormatArgsTrace::EncodeArgumentsWithArray(Buffer, FormatArgs...);
		const uint8* FormatArgsBuffer = Buffer.GetData();
		if (FormatArgsSize && FormatArgsSize < INT32_MAX)
		{
			OutputLogMessageInternal(LogPoint, FormatArgsSize, FormatArgsBuffer);
		}
	}

private:
	CORE_API static void OutputLogMessageInternal(const void* LogPoint, int32 EncodedFormatArgsSize, const uint8* EncodedFormatArgs);
};

#define TRACE_LOG_CATEGORY(Category, Name, DefaultVerbosity) \
	FLogTrace::OutputLogCategory(Category, Name, DefaultVerbosity);

#if LOGTRACE_RUNTIME_FORMATTING_ENABLED
#define TRACE_LOG_MESSAGE(Category, Verbosity, Format, ...) \
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(LogChannel)) \
	{ \
		static bool UE_JOIN(__LogPoint, __LINE__); \
		if (!UE_JOIN(__LogPoint, __LINE__)) \
		{ \
			FLogTrace::OutputLogMessageSpec(&UE_JOIN(__LogPoint, __LINE__), &Category, ELogVerbosity::Verbosity, __FILE__, __LINE__, TEXT("%s")); \
			UE_JOIN(__LogPoint, __LINE__) = true; \
		} \
		FLogTrace::OutputLogMessageSimple(&UE_JOIN(__LogPoint, __LINE__), Format, ##__VA_ARGS__); \
	}
#else
#define TRACE_LOG_MESSAGE(Category, Verbosity, Format, ...) \
	static_assert(TIsArrayOrRefOfTypeByPredicate<decltype(Format), TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array."); \
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(LogChannel)) \
	{ \
		static bool UE_JOIN(__LogPoint, __LINE__); \
		if (!UE_JOIN(__LogPoint, __LINE__)) \
		{ \
			FLogTrace::OutputLogMessageSpec(&UE_JOIN(__LogPoint, __LINE__), &Category, ELogVerbosity::Verbosity, __FILE__, __LINE__, (const TCHAR*)Format); \
			UE_JOIN(__LogPoint, __LINE__) = true; \
		} \
		FLogTrace::OutputLogMessage(&UE_JOIN(__LogPoint, __LINE__), ##__VA_ARGS__); \
	}
#endif

#else
#define TRACE_LOG_CATEGORY(Category, Name, DefaultVerbosity)
#define TRACE_LOG_MESSAGE(Category, Verbosity, Format, ...)
#endif
