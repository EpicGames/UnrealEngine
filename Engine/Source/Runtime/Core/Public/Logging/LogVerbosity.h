// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class FString;

/** 
 * Enum that defines the verbosity levels of the logging system.
 * Also defines some non-verbosity levels that are hacks that allow
 * breaking on a given log line or setting the color.
**/
namespace ELogVerbosity
{
	enum Type : uint8
	{
		/** Not used */
		NoLogging		= 0,

		/** Always prints a fatal error to console (and log file) and crashes (even if logging is disabled) */
		Fatal,

		/** 
		 * Prints an error to console (and log file). 
		 * Commandlets and the editor collect and report errors. Error messages result in commandlet failure.
		 */
		Error,

		/** 
		 * Prints a warning to console (and log file).
		 * Commandlets and the editor collect and report warnings. Warnings can be treated as an error.
		 */
		Warning,

		/** Prints a message to console (and log file) */
		Display,

		/** Prints a message to a log file (does not print to console) */
		Log,

		/** 
		 * Prints a verbose message to a log file (if Verbose logging is enabled for the given category, 
		 * usually used for detailed logging) 
		 */
		Verbose,

		/** 
		 * Prints a verbose message to a log file (if VeryVerbose logging is enabled, 
		 * usually used for detailed logging that would otherwise spam output) 
		 */
		VeryVerbose,

		// Log masks and special Enum values

		All				= VeryVerbose,
		NumVerbosity,
		VerbosityMask	= 0xf,
		SetColor		= 0x40, // not actually a verbosity, used to set the color of an output device 
		BreakOnLog		= 0x80
	};
}

static_assert(ELogVerbosity::NumVerbosity - 1 < ELogVerbosity::VerbosityMask, "Bad verbosity mask.");
static_assert(!(ELogVerbosity::VerbosityMask & ELogVerbosity::BreakOnLog), "Bad verbosity mask.");

/**
 * Converts verbosity to a string
 * @param Verbosity verbosity enum
 * @returns String representation of the verbosity enum
 */
CORE_API const TCHAR* ToString(ELogVerbosity::Type Verbosity);

/**
 * Converts a string to verbosity 
 * @param VerbosityString verbosity in string form (e.g., "Log", "Display", or "Fatal")
 * @returns Parsed verbosity from the string (or Log if the string was invalid)
 */
extern CORE_API ELogVerbosity::Type ParseLogVerbosityFromString(const FString& VerbosityString);

