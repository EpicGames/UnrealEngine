// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/SortedMap.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Logging/LogVerbosity.h"
#include "Logging/TokenizedMessage.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "HAL/LowLevelMemTracker.h"

class IMessageLog;

LLM_DECLARE_TAG_API(EngineMisc_MessageLog, CORE_API);

class FMessageLog
{
public:
	/**
	 * Constructor
	 * @param	InLogName	the name of the log to use
	 */
	CORE_API FMessageLog( const FName& InLogName );

	/**
	 * Destructor
	 * When this object goes out of scope, its buffered messages will be flushed.
	 */
	CORE_API ~FMessageLog();

	/** Send the currently buffered messages to the log & clear the buffer */
	CORE_API void Flush();

	/**
	 * Add a message to the log.
	 * @param	InMessage	The message to add
	 * @return The message for chaining calls
	 */
	CORE_API const TSharedRef<FTokenizedMessage>& AddMessage( const TSharedRef<FTokenizedMessage>& InMessage );

	/**
	 * Add an array of messages to the message log
	 * @param	InMessages	The messages to add
	 */
	CORE_API void AddMessages( const TArray< TSharedRef<FTokenizedMessage> >& InMessages );

	/** 
	 * Helper functions to allow easy adding of messages to the log. Each one will add a token
	 * for the severity (if valid) and the textual message (if any). Messages are buffered.
	 * @param	InSeverity	The severity of the message
	 * @param	InMessage	The message to use
	 * @return the message for chaining calls.
	 */
	CORE_API TSharedRef<FTokenizedMessage> Message( EMessageSeverity::Type InSeverity, const FText& InMessage = FText() );
	CORE_API TSharedRef<FTokenizedMessage> Error( const FText& InMessage = FText() );
	CORE_API TSharedRef<FTokenizedMessage> PerformanceWarning( const FText& InMessage = FText() );
	CORE_API TSharedRef<FTokenizedMessage> Warning( const FText& InMessage = FText() );
	CORE_API TSharedRef<FTokenizedMessage> Info( const FText& InMessage = FText() );

	/**
	 * Check whether there are any messages present for this log.
	 * If the log is paged, this will check if there are any messages on the current page.
	 * This call will cause a flush so that the logs state is properly reflected.
	 * @param	InSeverityFilter	Only messages of higher severity than this filter will be considered when checking.
	 * @return The number of messages that pass our filter
	 */
	CORE_API int32 NumMessages( EMessageSeverity::Type InSeverityFilter = EMessageSeverity::Info );

	/**
	 * Opens the log for display to the user given certain conditions.
	 * This call will cause a flush so that the logs state is properly reflected.
	 * @param	InSeverityFilter	Only messages of higher severity than this filter will be considered when checking.
	 * @param	bOpenEvenIfEmpty	Override the filter & log status & force the log to open.
	 */
	CORE_API void Open( EMessageSeverity::Type InSeverityFilter = EMessageSeverity::Info, bool bOpenEvenIfEmpty = false );

	/**
	 * Notify the user with a message if there are messages present.
	 * This call will cause a flush so that the logs state is properly reflected.
	 * @param	InMessage			The notification message.
	 * @param	InSeverityFilter	Only messages of higher severity than this filter will be considered when checking.
	 * @param	bForce				Notify anyway, even if the filters gives us no messages.
	 */
	CORE_API void Notify( const FText& InMessage = FText(), EMessageSeverity::Type InSeverityFilter = EMessageSeverity::Info, bool bForce = false );

	/**
	 * Add a new page to the log.
	 * Pages will not be created until messages are added to them, so we dont generate lots of empty pages.
	 * This call will cause a flush so that the logs state is properly reflected.
	 * @param	InLabel		The label for the page.
	 */
	CORE_API void NewPage( const FText& InLabel );

	/**
	* Sets the current page to the one specified by the label.
	* This call will cause a flush so that the logs state is properly reflected.
	* @param	InLabel		The label for the page.
	*/
	CORE_API void SetCurrentPage( const FText& InLabel );

	/** Should we mirror message log messages from this instance to the output log during flush? */
	CORE_API FMessageLog& SuppressLoggingToOutputLog(bool bShouldSuppress = true);

	/**
	 * Delegate to retrieve a log interface.
	 * This allows systems that implement IMessageLog to receive messages.
	 */
	DECLARE_DELEGATE_RetVal_OneParam( TSharedRef<class IMessageLog>, FGetLog, const FName& );
	static FGetLog& OnGetLog() { return GetLog; }

	/**
	 * Delegate used when message selection changes.
	 * This is used to select object in the scene according to objects referenced in message tokens.
	 */
	DECLARE_DELEGATE_OneParam( FMessageSelectionChanged, TArray< TSharedRef<FTokenizedMessage> >& );
	static FMessageSelectionChanged& OnMessageSelectionChanged() { return MessageSelectionChanged; }

	/** Helper function to convert message log severity to log verbosity */
	CORE_API static ELogVerbosity::Type GetLogVerbosity( EMessageSeverity::Type InSeverity );

	/** Helper function to retrieve a log color for a specified message severity */
	CORE_API static const TCHAR* const GetLogColor( EMessageSeverity::Type InSeverity );

	/** Delegates for multi-select of message token actors */
	DECLARE_DELEGATE(FMultiSelectActorBegin);
	static FMultiSelectActorBegin& OnMultiSelectActorBegin() { return MultiSelectActorBegin; }

	DECLARE_DELEGATE(FMultiSelectActorEnd);
	static FMultiSelectActorEnd& OnMultiSelectActorEnd() { return MultiSelectActorEnd; }

private:
	/** Buffer for messages */
	TArray< TSharedRef<FTokenizedMessage> > Messages;

	/** The message log we use for output */
	TSharedPtr<class IMessageLog> MessageLog;

	/** Name of this message log */
	FName LogName;

	/** Do we want to suppress logging to the output log in addition to the message log? */
	bool bSuppressLoggingToOutputLog;

	/** Delegate for retrieving the message log */
	CORE_API static FGetLog GetLog;

	/** Delegate for message selection */
	CORE_API static FMessageSelectionChanged MessageSelectionChanged;

	/** Delegates for multi-select of message token actors */
	CORE_API static FMultiSelectActorBegin MultiSelectActorBegin;
	CORE_API static FMultiSelectActorEnd MultiSelectActorEnd;
};

/**
 * Scoped override for FMessageLog behavior.
 * This can override the log behavior of a given named FMessageLog for the duration of the overrides lifetime, optionally suppressing 
 * output log mirroring, or promoting/demoting log categories (eg, to make errors act as warnings, or warnings act as errors).
 */
class FMessageLogScopedOverride final
{
public:
	CORE_API explicit FMessageLogScopedOverride(const FName InLogName);
	CORE_API ~FMessageLogScopedOverride();

	UE_NONCOPYABLE(FMessageLogScopedOverride);

	/** Should we mirror message log messages to the output log during flush? */
	CORE_API FMessageLogScopedOverride& SuppressLoggingToOutputLog(const bool bShouldSuppress = true);

	/** Map category X to category Y when adding messages to this log */
	CORE_API FMessageLogScopedOverride& RemapMessageSeverity(const EMessageSeverity::Type SrcSeverity, const EMessageSeverity::Type DestSeverity);

private:
	friend class FMessageLog;

	FName LogName;
	TOptional<bool> bSuppressLoggingToOutputLog;
	TSortedMap<EMessageSeverity::Type, EMessageSeverity::Type> MessageSeverityRemapping;
};
