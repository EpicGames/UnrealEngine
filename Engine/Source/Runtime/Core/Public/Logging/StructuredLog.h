// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/Insertable.h"
#include "Containers/StringFwd.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"
#include "Misc/OptionalFwd.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Templates/IsArrayOrRefOfType.h"
#include "Templates/Models.h"
#include "Templates/Requires.h"
#include "Templates/UnrealTypeTraits.h"

#include <atomic>
#include <type_traits>

class FTextFormat;
template <typename FuncType> class TFunctionRef;

#define UE_API CORE_API

/**
 * Records a structured log event if this category is active at this level of verbosity.
 *
 * Supports either positional or named parameters, but not a mix of these styles.
 *
 * Positional: The field values must exactly match the fields referenced by Format.
 * UE_LOGFMT(LogCore, Warning, "Loading '{Name}' failed with error {Error}", Package->GetName(), ErrorCode);
 *
 * Named: The fields must contain every field referenced by Format. Order is irrelevant and extra fields are permitted.
 * UE_LOGFMT(LogCore, Warning, "Loading '{Name}' failed with error {Error}",
 *     ("Name", Package->GetName()), ("Error", ErrorCode), ("Flags", LoadFlags));
 *
 * Field names must match "[A-Za-z0-9_]+" and must be unique within this log event.
 * Field values will be serialized using SerializeForLog or operator<<(FCbWriter&, FieldType).
 *
 * @param CategoryName   Name of a log category declared by DECLARE_LOG_CATEGORY_*.
 * @param Verbosity      Name of a log verbosity level from ELogVerbosity.
 * @param Format         Format string in the style of FLogTemplate.
 * @param Fields[0-16]   Zero to sixteen fields or field values.
 */
#define UE_LOGFMT(CategoryName, Verbosity, Format, ...) UE_PRIVATE_LOGFMT_CALL(UE_LOGFMT_EX, (CategoryName, Verbosity, Format UE_PRIVATE_LOGFMT_FIELDS(__VA_ARGS__)))
 
 /** Conditional UE_LOGFMT. */
#define UE_CLOGFMT(Condition, CategoryName, Verbosity, Format, ...) UE_PRIVATE_LOGFMT_CALL(UE_CLOGFMT_EX, (Condition, CategoryName, Verbosity, Format UE_PRIVATE_LOGFMT_FIELDS(__VA_ARGS__)))

/**
 * Records a structured log event if this category is active at this level of verbosity.
 *
 * This has the same functionality as UE_LOGFMT but removes the limit on field count.
 *
 * Positional: Values must be wrapped in UE_LOGFMT_VALUE.
 * UE_LOGFMT_EX(LogCore, Warning, "Loading '{Name}' failed with error {Error}",
 *     UE_LOGFMT_VALUE(Package->GetName()), UE_LOGFMT_VALUE(ErrorCode));
 *
 * Named: Fields must be wrapped in UE_LOGFMT_FIELD.
 * UE_LOGFMT_EX(LogCore, Warning, "Loading '{Name}' failed with error {Error}",
 *     UE_LOGFMT_FIELD("Name", Package->GetName()), UE_LOGFMT_FIELD("Error", ErrorCode), UE_LOGFMT_FIELD("Flags", LoadFlags));
 */
#define UE_LOGFMT_EX(CategoryName, Verbosity, Format, ...) UE_PRIVATE_LOGFMT(UE_EMPTY, CategoryName, Verbosity, Format, ##__VA_ARGS__)

/** Conditional UE_LOGFMT_EX. */
#define UE_CLOGFMT_EX(Condition, CategoryName, Verbosity, Format, ...) UE_PRIVATE_LOGFMT(if (Condition), CategoryName, Verbosity, Format, ##__VA_ARGS__)

/**
 * Records a localized structured log event if this category is active at this level of verbosity.
 *
 * Example:
 * UE_LOGFMT_LOC(LogCore, Warning, "LoadingFailed", "Loading '{Name}' failed with error {Error}",
 *     ("Name", Package->GetName()), ("Error", ErrorCode), ("Flags", LoadFlags));
 *
 * Field names must match "[A-Za-z0-9_]+" and must be unique within this log event.
 * Field values will be serialized using SerializeForLog or operator<<(FCbWriter&, FieldType).
 * The fields must contain every field referenced by Format. Order is irrelevant and extra fields are permitted.
 *
 * @param CategoryName   Name of a log category declared by DECLARE_LOG_CATEGORY_*.
 * @param Verbosity      Name of a log verbosity level from ELogVerbosity.
 * @param Namespace      Namespace for the format FText, or LOCTEXT_NAMESPACE for the non-NS macro.
 * @param Key            Key for the format FText that is unique within the namespace.
 * @param Format         Format string in the style of FTextFormat.
 * @param Fields[0-16]   Zero to sixteen fields in the format ("Name", Value).
 */
#define UE_LOGFMT_LOC(CategoryName, Verbosity, Key, Format, ...) \
	UE_LOGFMT_NSLOC(CategoryName, Verbosity, LOCTEXT_NAMESPACE, Key, Format, ##__VA_ARGS__)
#define UE_LOGFMT_NSLOC(CategoryName, Verbosity, Namespace, Key, Format, ...) \
	UE_PRIVATE_LOGFMT_CALL(UE_LOGFMT_NSLOC_EX, (CategoryName, Verbosity, Namespace, Key, Format UE_PRIVATE_LOGFMT_FIELDS(__VA_ARGS__)))

/** Conditional UE_LOGFMT_LOC. */
#define UE_CLOGFMT_LOC(Condition, CategoryName, Verbosity, Key, Format, ...) \
	UE_CLOGFMT_NSLOC(Condition, CategoryName, Verbosity, LOCTEXT_NAMESPACE, Key, Format, ##__VA_ARGS__)
#define UE_CLOGFMT_NSLOC(Condition, CategoryName, Verbosity, Namespace, Key, Format, ...) \
	UE_PRIVATE_LOGFMT_CALL(UE_CLOGFMT_NSLOC_EX, (Condition, CategoryName, Verbosity, Namespace, Key, Format UE_PRIVATE_LOGFMT_FIELDS(__VA_ARGS__)))

/**
 * Records a localized structured log event if this category is active at this level of verbosity.
 *
 * Example:
 * UE_LOGFMT_LOC_EX(LogCore, Warning, "LoadingFailed", "Loading '{Name}' failed with error {Error}",
 *     UE_LOGFMT_FIELD("Name", Package->GetName()), UE_LOGFMT_FIELD("Error", ErrorCode), UE_LOGFMT_FIELD("Flags", LoadFlags));
 *
 * Same as UE_LOGFMT_LOC and works for any number of fields.
 * Fields must be written as UE_LOGFMT_FIELD("Name", Value).
 */
#define UE_LOGFMT_LOC_EX(CategoryName, Verbosity, Key, Format, ...) \
	UE_LOGFMT_NSLOC_EX(CategoryName, Verbosity, LOCTEXT_NAMESPACE, Key, Format, ##__VA_ARGS__)
#define UE_LOGFMT_NSLOC_EX(CategoryName, Verbosity, Namespace, Key, Format, ...) \
	UE_PRIVATE_LOGFMT_LOC(UE_EMPTY, CategoryName, Verbosity, Namespace, Key, Format, ##__VA_ARGS__)

/** Conditional UE_LOGFMT_LOC_EX. */
#define UE_CLOGFMT_LOC_EX(Condition, CategoryName, Verbosity, Key, Format, ...) \
	UE_CLOGFMT_NSLOC_EX(Condition, CategoryName, Verbosity, LOCTEXT_NAMESPACE, Key, Format, ##__VA_ARGS__)
#define UE_CLOGFMT_NSLOC_EX(Condition, CategoryName, Verbosity, Namespace, Key, Format, ...) \
	UE_PRIVATE_LOGFMT_LOC(if (Condition), CategoryName, Verbosity, Namespace, Key, Format, ##__VA_ARGS__)

/** Expands to a named structured log field. UE_LOGFMT_FIELD("Name", Value) */
#define UE_LOGFMT_FIELD(Name, Value) UE_PRIVATE_LOGFMT_FIELD((Name, Value))

/** Expands to a structured log value. UE_LOGFMT_VALUE(Value) */
#define UE_LOGFMT_VALUE(Value) Value

#if !NO_LOGGING

/**
 * Registers a context on the calling thread, by name, with an optional value.
 *
 * The optional value can be anything that serializes to compact binary using FCbWriter or SerializeForLog.
 * Context is unregistered when it goes out of scope.
 * Context is overridden by a newer context with the same name.
 * Context is overridden by a field of the same name in the log record.
 * Context is copied to FLogRecord::Fields for every log record created in the lifetime of the context.
 * Context names are written to FLogRecord::Fields in an array named $Context.
 * Contexts can be visited by calling VisitLogContext.
 *
 * Examples:
 * UE_LOG_CONTEXT("Loading");
 * UE_LOG_CONTEXT("Count", 123.0);
 * UE_LOG_CONTEXT("Asset", FAssetLog(AssetPath));
 */
#define UE_LOG_CONTEXT(NameLiteral, ...) ::UE::Logging::Private::FLogContext ANONYMOUS_VARIABLE(LogContext_)(::UE::Logging::Private::CheckFieldName(NameLiteral).Name, ##__VA_ARGS__)

#else // #if !NO_LOGGING

#define UE_LOG_CONTEXT(...)

#endif

struct FDateTime;

namespace UE
{

/** Template format is "Text with {Fields} embedded {Like}{This}. {{Double to escape.}}" */
class FLogTemplate;

/**
 * Time that a log event occurred.
 */
class FLogTime
{
public:
	UE_API static FLogTime Now();
	UE_API static FLogTime FromUtcTime(const FDateTime& UtcTime);

	constexpr FLogTime() = default;

	/** Returns the UTC time. 0 ticks when the time was not set. */
	UE_API FDateTime GetUtcTime() const;

private:
	/** Ticks from FDateTime::UtcNow() */
	int64 UtcTicks = 0;
};

/**
 * Record of a log event.
 */
class FLogRecord
{
public:
	/** The optional name of the category for the log record. None when omitted. */
	const FName& GetCategory() const { return Category; }
	void SetCategory(const FName& InCategory) { Category = InCategory; }

	/** The verbosity level of the log record. Must be a valid level with no flags or special values. */
	ELogVerbosity::Type GetVerbosity() const { return Verbosity; }
	void SetVerbosity(ELogVerbosity::Type InVerbosity) { Verbosity = InVerbosity; }

	/** The time at which the log record was created. */
	const FLogTime& GetTime() const { return Time; }
	void SetTime(const FLogTime& InTime) { Time = InTime; }

	/** The format string that serves as the message for the log record. Example: TEXT("FieldName is {FieldName}") */
	UE_DEPRECATED(5.8, "Use GetUtf8Format.")
	const TCHAR* GetFormat() const { return Format; }
	UE_DEPRECATED(5.8, "Use SetUtf8Format.")
	void SetFormat(const TCHAR* InFormat) { Format = InFormat; }

	/** The format string that serves as the message for the log record. Example: UTF8TEXT("FieldName is {FieldName}") */
	const UTF8CHAR* GetUtf8Format() const { return Utf8Format; }
	void SetUtf8Format(const UTF8CHAR* InFormat) { Utf8Format = InFormat; }

	/** The optional template for the format string. */
	const FLogTemplate* GetTemplate() const { return Template; }
	void SetTemplate(const FLogTemplate* InTemplate) { Template = InTemplate; }

	/** The fields referenced by the format string, along with optional additional fields. */
	const FCbObject& GetFields() const { return Fields; }
	void SetFields(FCbObject&& InFields) { Fields = MoveTemp(InFields); }

	/** The optional source file path for the code that created the log record. Null when omitted. */
	const ANSICHAR* GetFile() const { return File; }
	void SetFile(const ANSICHAR* InFile) { File = InFile; }

	/** The optional source line number for the code that created the log record. 0 when omitted. */
	int32 GetLine() const { return Line; }
	void SetLine(int32 InLine) { Line = InLine; }

	/** The namespace of the localized text. Null when non-localized. */
	const TCHAR* GetTextNamespace() const { return TextNamespace; }
	void SetTextNamespace(const TCHAR* InTextNamespace) { TextNamespace = InTextNamespace; }

	/** The key of the localized text. Null when non-localized. */
	const TCHAR* GetTextKey() const { return TextKey; }
	void SetTextKey(const TCHAR* InTextKey) { TextKey = InTextKey; }

	/** Formats the message using the format, template, and fields. */
	UE_API void FormatMessageTo(FUtf8StringBuilderBase& Out) const;
	UE_API void FormatMessageTo(FWideStringBuilderBase& Out) const;

	/**
	 * Converts this record into a common format string and compatible fields.
	 *
	 * The common format string uses '{{' as a literal '{' and uses as a literal '}}'.
	 * The common format string contains no format specifiers or format argument modifiers.
	 * The common format string uses field names matching [A-Za-z0-9_]+.
	 *
	 * When a compatible field is an object, it will have a string field named $text containing the formatted value.
	 * A compatible field will never be an array.
	 *
	 * @param OutFormat   Appended with a transformed version of the format from Template.
	 * @param OutFields   Must be within an object scope. Appended with a compatible version of Fields.
	 */
	UE_API void ConvertToCommonLog(FUtf8StringBuilderBase& OutFormat, FCbWriter& OutFields) const;

private:
	const TCHAR* Format = nullptr;
	const UTF8CHAR* Utf8Format = nullptr;
	const ANSICHAR* File = nullptr;
	int32 Line = 0;
	FName Category;
	ELogVerbosity::Type Verbosity = ELogVerbosity::Log;
	FLogTime Time;
	FCbObject Fields;
	const FLogTemplate* Template = nullptr;
	const TCHAR* TextNamespace = nullptr;
	const TCHAR* TextKey = nullptr;
};

/**
 * Advanced function to dispatch a log record to active output devices.
 *
 * Always use UE_LOGFMT or its variants when possible.
 * Dynamic dispatch bypasses many optimizations provided by the macros.
 * Anything pointed to by the record MUST remain valid until threaded logs have been flushed.
 * Filtering by category or verbosity is the responsibility of the caller.
 * Active log contexts will not be added to this record. If they are needed that is the responsibility of the caller.
 */
UE_API void DispatchDynamicLogRecord(const FLogRecord& Record);

/**
 * Advanced function to visit the log contexts for the calling thread.
 *
 * Invokes the visitor once for each context, in order from the oldest context added on this thread to the newest.
 * If there are multiple contexts with the same name, only the newest will be visited.
 */
UE_API void VisitLogContext(TFunctionRef<void (const FCbField&)> Visitor);

/**
 * Serializes the value to be used in a log message.
 *
 * Overload this when the log behavior needs to differ from general serialization to compact binary.
 *
 * There are three ways to perform custom formatting for values that are serialized as an object:
 * 1. Add a $text field that is a string with the exact text to display.
 * 2. Add a $format field that is a format string that may reference fields of the object and their sub-objects.
 * 3. Add a $locformat field that is a localized format string that may reference fields of the object and their
 *    sub-objects. The namespace and key must be included in $locns and $lockey string fields. Serialize FText
 *    to the writer using SerializeLogFormat().
 *
 * Arrays and objects without custom formatting are converted to JSON.
 */
template <CInsertableInto<FCbWriter&> ValueType>
inline void SerializeForLog(FCbWriter& Writer, ValueType&& Value)
{
	Writer << (ValueType&&)Value;
}

/** Describes a type that can be serialized for use in a log message. */
struct CSerializableForLog
{
	template <typename ValueType>
	auto Requires(FCbWriter& Writer, ValueType&& Value) -> decltype(
		SerializeForLog(Writer, (ValueType&&)Value)
	);
};

/** Wrapper to support calling SerializeForLog with ADL from within an overload of SerializeForLog. */
template <typename ValueType UE_REQUIRES(TModels_V<CSerializableForLog, ValueType>)>
inline void CallSerializeForLog(FCbWriter& Writer, ValueType&& Value)
{
	SerializeForLog(Writer, (ValueType&&)Value);
}

} // UE

template <typename ValueType UE_REQUIRES(TModels_V<UE::CSerializableForLog, ValueType>)>
inline void SerializeForLog(FCbWriter& Writer, const TOptional<ValueType>& Optional)
{
	Writer.BeginArray();
	if (Optional)
	{
		UE::CallSerializeForLog(Writer, Optional.GetValue());
	}
	Writer.EndArray();
}

namespace UE::Logging::Private
{

/** Data about a static log that is created on-demand. */
struct FStaticLogDynamicData
{
#if LOGTRACE_ENABLED
	std::atomic<bool> bInitializedTrace = false;
#endif
};

/** Data about a static log that is constant for every occurrence. */
struct FStaticLogRecord
{
	const UTF8CHAR* Format = nullptr;
	UE_IF(UE_LOG_INCLUDE_SOURCE_LOCATION,, inline static constexpr) const ANSICHAR* File = nullptr;
	UE_IF(UE_LOG_INCLUDE_SOURCE_LOCATION,, inline static constexpr) int32 Line = 0;
	ELogVerbosity::Type Verbosity = ELogVerbosity::Log;
#if LOGTRACE_ENABLED
	FStaticLogDynamicData& DynamicData;
#endif

	// Workaround for https://developercommunity.visualstudio.com/t/Incorrect-warning-C4700-with-unrelated-s/10285950
	constexpr FStaticLogRecord(
		const UTF8CHAR* InFormat,
		const ANSICHAR* InFile,
		int32 InLine,
		ELogVerbosity::Type InVerbosity,
		FStaticLogDynamicData* InDynamicData)
		: Format(InFormat)
	#if UE_LOG_INCLUDE_SOURCE_LOCATION
		, File(InFile)
		, Line(InLine)
	#endif
		, Verbosity(InVerbosity)
	#if LOGTRACE_ENABLED
		, DynamicData(*InDynamicData)
	#endif
	{
	}
};

/** Data about a static localized log that is constant for every occurrence. */
struct FStaticLocalizedLogRecord
{
	const TCHAR* TextNamespace = nullptr;
	const TCHAR* TextKey = nullptr;
	const UTF8CHAR* Format = nullptr;
	UE_IF(UE_LOG_INCLUDE_SOURCE_LOCATION,, inline static constexpr) const ANSICHAR* File = nullptr;
	UE_IF(UE_LOG_INCLUDE_SOURCE_LOCATION,, inline static constexpr) int32 Line = 0;
	ELogVerbosity::Type Verbosity = ELogVerbosity::Log;
#if LOGTRACE_ENABLED
	FStaticLogDynamicData& DynamicData;
#endif

	// Workaround for https://developercommunity.visualstudio.com/t/Incorrect-warning-C4700-with-unrelated-s/10285950
	constexpr FStaticLocalizedLogRecord(
		const TCHAR* InTextNamespace,
		const TCHAR* InTextKey,
		const UTF8CHAR* InFormat,
		const ANSICHAR* InFile,
		int32 InLine,
		ELogVerbosity::Type InVerbosity,
		FStaticLogDynamicData* InDynamicData)
		: TextNamespace(InTextNamespace)
		, TextKey(InTextKey)
		, Format(InFormat)
	#if UE_LOG_INCLUDE_SOURCE_LOCATION
		, File(InFile)
		, Line(InLine)
	#endif
		, Verbosity(InVerbosity)
	#if LOGTRACE_ENABLED
		, DynamicData(*InDynamicData)
	#endif
	{
	}
};

struct FLogField
{
	using FWriteFn = void (FCbWriter& Writer, const void* Value);

	const ANSICHAR* Name;
	const void* Value;
	FWriteFn* WriteValue;

	template <typename ValueType>
	static void Write(FCbWriter& Writer, const void* Value)
	{
		SerializeForLog(Writer, *(const ValueType*)Value);
	}
};

#if !NO_LOGGING
UE_API void LogWithNoFields(const FLogCategoryBase& Category, const FStaticLogRecord& Log);
UE_API void LogWithFieldArray(const FLogCategoryBase& Category, const FStaticLogRecord& Log, const FLogField* Fields, int32 FieldCount);
UE_API void LogWithNoFields(const FLogCategoryBase& Category, const FStaticLocalizedLogRecord& Log);
UE_API void LogWithFieldArray(const FLogCategoryBase& Category, const FStaticLocalizedLogRecord& Log, const FLogField* Fields, int32 FieldCount);
#endif

[[noreturn]] UE_API void FatalLogWithNoFields(const FLogCategoryBase& Category, const FStaticLogRecord& Log);
[[noreturn]] UE_API void FatalLogWithFieldArray(const FLogCategoryBase& Category, const FStaticLogRecord& Log, const FLogField* Fields, int32 FieldCount);
[[noreturn]] UE_API void FatalLogWithNoFields(const FLogCategoryBase& Category, const FStaticLocalizedLogRecord& Log);
[[noreturn]] UE_API void FatalLogWithFieldArray(const FLogCategoryBase& Category, const FStaticLocalizedLogRecord& Log, const FLogField* Fields, int32 FieldCount);

/** Wrapper to identify field names interleaved with field values. */
template <typename NameType>
struct TLogFieldName
{
	NameType Name;
};

/** Verify that the name is likely a string literal and forward it on. */
template <typename NameType>
inline constexpr TLogFieldName<NameType> CheckFieldName(NameType&& Name)
{
	static_assert(TIsArrayOrRefOfType<NameType, ANSICHAR>::Value, "Name must be an ANSICHAR string literal.");
	return {(NameType&&)Name};
}

/** Create log fields from values optionally preceded by names. */
struct FLogFieldCreator
{
	template <typename T> inline constexpr static int32 ValueCount = 1;
	template <typename T> inline constexpr static int32 ValueCount<TLogFieldName<T>> = 0;

	template <typename... FieldArgTypes>
	inline constexpr static int32 GetCount()
	{
		return (ValueCount<FieldArgTypes> + ...);
	}

	inline static void Create(FLogField* Fields)
	{
	}

	template <typename ValueType, typename... FieldArgTypes, std::enable_if_t<ValueCount<ValueType>>* = nullptr>
	inline static void Create(FLogField* Fields, const ValueType& Value, FieldArgTypes&&... FieldArgs)
	{
		new(Fields) FLogField{nullptr, &Value, FLogField::Write<ValueType>};
		Create(Fields + 1, (FieldArgTypes&&)FieldArgs...);
	}

	template <typename NameType, typename ValueType, typename... FieldArgTypes>
	inline static void Create(FLogField* Fields, TLogFieldName<NameType> Name, const ValueType& Value, FieldArgTypes&&... FieldArgs)
	{
		new(Fields) FLogField{Name.Name, &Value, FLogField::Write<ValueType>};
		Create(Fields + 1, (FieldArgTypes&&)FieldArgs...);
	}
};

/** Log with fields created from the arguments, which may be values or pairs of name/value. */
template <typename LogType, typename... FieldArgTypes>
UE_COLD UE_DEBUG_SECTION void LogWithFields(const FLogCategoryBase& Category, const LogType& Log, FieldArgTypes&&... FieldArgs)
{
	if constexpr (sizeof...(FieldArgTypes) == 0)
	{
		LogWithNoFields(Category, Log);
	}
	else
	{
		constexpr int32 FieldCount = FLogFieldCreator::template GetCount<FieldArgTypes...>();
		static_assert(FieldCount > 0);
		FLogField Fields[FieldCount];
		FLogFieldCreator::Create(Fields, (FieldArgTypes&&)FieldArgs...);
		LogWithFieldArray(Category, Log, Fields, FieldCount);
	}
}

/** Fatal log with fields created from the arguments, which may be values or pairs of name/value. */
template <typename LogType, typename... FieldArgTypes>
[[noreturn]] UE_COLD UE_DEBUG_SECTION void FatalLogWithFields(const FLogCategoryBase& Category, const LogType& Log, FieldArgTypes&&... FieldArgs)
{
	if constexpr (sizeof...(FieldArgTypes) == 0)
	{
		FatalLogWithNoFields(Category, Log);
	}
	else
	{
		constexpr int32 FieldCount = FLogFieldCreator::template GetCount<FieldArgTypes...>();
		static_assert(FieldCount > 0);
		FLogField Fields[FieldCount];
		FLogFieldCreator::Create(Fields, (FieldArgTypes&&)FieldArgs...);
		FatalLogWithFieldArray(Category, Log, Fields, FieldCount);
	}
}

#if NO_LOGGING
UE_API extern FLogCategory<ELogVerbosity::Fatal, ELogVerbosity::Fatal> LogFatal;
#endif

struct FLogContext
{
	template <typename ValueType>
	inline explicit FLogContext(const ANSICHAR* InName, const ValueType& InValue)
		: FLogContext(InName, &InValue, FLogField::Write<ValueType>)
	{
	}

	UE_API explicit FLogContext(const ANSICHAR* Name);
	UE_API explicit FLogContext(const ANSICHAR* Name, const void* Value, FLogField::FWriteFn* WriteValue);
	UE_API ~FLogContext();

	FLogContext(const FLogContext&) = delete;
	FLogContext& operator=(const FLogContext&) = delete;

	const ANSICHAR* Name = nullptr;
	TArray<uint8> Value;
	FLogContext* Next = nullptr;
	FLogContext* Prev = nullptr;
	bool bDisabledByNewerContext = false;
	bool bDisabledOlderContext = false;
};

// Compatibility function for FDebug::LogFormattedMessageWithCallstack to log a single line without checking log channel verbosity, accepting channel by name
// This exists to be able to emit those lines as structured logs inheriting the log context fields
void LogDebugMessageLine(const ANSICHAR* File, int32 Line, const FName& InLogName, ELogVerbosity::Type Verbosity, const TCHAR* Message, const TCHAR* Prefix = TEXT(""));
} // UE::Logging::Private

#if !NO_LOGGING
#define UE_PRIVATE_LOG_CATEGORY(CategoryName) CategoryName
#else
#define UE_PRIVATE_LOG_CATEGORY(CategoryName) ::UE::Logging::Private::LogFatal
#endif

#define UE_PRIVATE_LOGFMT(Condition, CategoryName, Verbosity, Format, ...) \
	do \
	{ \
		if constexpr ((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) == ::ELogVerbosity::Fatal || \
			((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) <= ::ELogVerbosity::COMPILED_IN_MINIMUM_VERBOSITY && \
			 (::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) <= UE_PRIVATE_LOG_CATEGORY(CategoryName).CompileTimeVerbosity)) \
		{ \
			UE_IF(LOGTRACE_ENABLED, static ::UE::Logging::Private::FStaticLogDynamicData LOG_Dynamic, ); \
			static constexpr ::UE::Logging::Private::FStaticLogRecord LOG_Static UE_PRIVATE_LOGFMT_AGGREGATE(UTF8TEXT(Format), __builtin_FILE(), __builtin_LINE(), ::ELogVerbosity::Verbosity, UE_IF(LOGTRACE_ENABLED, &LOG_Dynamic, nullptr)); \
			UE_PRIVATE_LOGFMT_LOG_IF_ACTIVE(Condition, CategoryName, Verbosity, LOG_Static, ##__VA_ARGS__); \
		} \
	} \
	while (false)

#define UE_PRIVATE_LOGFMT_LOC(Condition, CategoryName, Verbosity, Namespace, Key, Format, ...) \
	do \
	{ \
		if constexpr ((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) == ::ELogVerbosity::Fatal || \
			((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) <= ::ELogVerbosity::COMPILED_IN_MINIMUM_VERBOSITY && \
			 (::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) <= UE_PRIVATE_LOG_CATEGORY(CategoryName).CompileTimeVerbosity)) \
		{ \
			UE_IF(LOGTRACE_ENABLED, static ::UE::Logging::Private::FStaticLogDynamicData LOG_Dynamic, ); \
			static constexpr ::UE::Logging::Private::FStaticLocalizedLogRecord LOG_Static UE_PRIVATE_LOGFMT_AGGREGATE(TEXT(Namespace), TEXT(Key), UTF8TEXT(Format), __builtin_FILE(), __builtin_LINE(), ::ELogVerbosity::Verbosity, UE_IF(LOGTRACE_ENABLED, &LOG_Dynamic, nullptr)); \
			UE_PRIVATE_LOGFMT_LOG_IF_ACTIVE(Condition, CategoryName, Verbosity, LOG_Static, ##__VA_ARGS__); \
		} \
	} \
	while (false)

#define UE_PRIVATE_LOGFMT_LOG_IF_ACTIVE(Condition, CategoryName, Verbosity, Log, ...) \
	if constexpr ((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) == ::ELogVerbosity::Fatal) \
	{ \
		Condition \
		{ \
			::UE::Logging::Private::FatalLogWithFields(UE_PRIVATE_LOG_CATEGORY(CategoryName), LOG_Static, ##__VA_ARGS__); \
		} \
	} \
	else if (!UE_PRIVATE_LOG_CATEGORY(CategoryName).IsSuppressed(::ELogVerbosity::Verbosity)) \
	{ \
		Condition \
		{ \
			::UE::Logging::Private::LogWithFields(UE_PRIVATE_LOG_CATEGORY(CategoryName), LOG_Static, ##__VA_ARGS__); \
		} \
	}

// This macro avoids having non-parenthesized commas in the log macros.
// Such commas can cause issues when a log macro is wrapped by another macro.
#define UE_PRIVATE_LOGFMT_AGGREGATE(...) {__VA_ARGS__}

// This macro expands a field from either `(Name, Value)` or `Value`
// A `(Name, Value)` field is converted to `CheckFieldName(Name), Value`
// A `Value` field is passed through as `Value`
#define UE_PRIVATE_LOGFMT_FIELD(Field) UE_PRIVATE_LOGFMT_FIELD_EXPAND(UE_PRIVATE_LOGFMT_NAMED_FIELD Field)
// This macro is only called when the field was parenthesized.
#define UE_PRIVATE_LOGFMT_NAMED_FIELD(Name, ...) UE_PRIVATE_LOGFMT_NAMED_FIELD ::UE::Logging::Private::CheckFieldName(Name), __VA_ARGS__
// The next three macros remove UE_PRIVATE_LOGFMT_NAMED_FIELD from the expanded expression.
#define UE_PRIVATE_LOGFMT_FIELD_EXPAND(...) UE_PRIVATE_LOGFMT_FIELD_EXPAND_INNER(__VA_ARGS__)
#define UE_PRIVATE_LOGFMT_FIELD_EXPAND_INNER(...) UE_PRIVATE_LOGFMT_STRIP_ ## __VA_ARGS__
#define UE_PRIVATE_LOGFMT_STRIP_UE_PRIVATE_LOGFMT_NAMED_FIELD

// This macro expands `Arg1, Arg2` to `UE_PRIVATE_LOGFMT_FIELD(Arg1), UE_PRIVATE_LOGFMT_FIELD(Arg2), ...`
// This macro expands `("Name1", Arg1), ("Name2", Arg2)` to `UE_PRIVATE_LOGFMT_FIELD(("Name1", Arg1)), UE_PRIVATE_LOGFMT_FIELD(("Name2", Arg2)), ...
#define UE_PRIVATE_LOGFMT_FIELDS(...) UE_PRIVATE_LOGFMT_CALL(UE_JOIN(UE_PRIVATE_LOGFMT_FIELDS_, UE_PRIVATE_LOGFMT_COUNT(__VA_ARGS__)), (__VA_ARGS__))

#define UE_PRIVATE_LOGFMT_FIELDS_0()
#define UE_PRIVATE_LOGFMT_FIELDS_1(A)                 , UE_PRIVATE_LOGFMT_FIELD(A)
#define UE_PRIVATE_LOGFMT_FIELDS_2(A,B)               , UE_PRIVATE_LOGFMT_FIELD(A), UE_PRIVATE_LOGFMT_FIELD(B)
#define UE_PRIVATE_LOGFMT_FIELDS_3(A,B,C)             , UE_PRIVATE_LOGFMT_FIELD(A), UE_PRIVATE_LOGFMT_FIELD(B), UE_PRIVATE_LOGFMT_FIELD(C)
#define UE_PRIVATE_LOGFMT_FIELDS_4(A,B,C,D)           , UE_PRIVATE_LOGFMT_FIELD(A), UE_PRIVATE_LOGFMT_FIELD(B), UE_PRIVATE_LOGFMT_FIELD(C), UE_PRIVATE_LOGFMT_FIELD(D)
#define UE_PRIVATE_LOGFMT_FIELDS_5(A,B,C,D,E)         , UE_PRIVATE_LOGFMT_FIELD(A), UE_PRIVATE_LOGFMT_FIELD(B), UE_PRIVATE_LOGFMT_FIELD(C), UE_PRIVATE_LOGFMT_FIELD(D), UE_PRIVATE_LOGFMT_FIELD(E)
#define UE_PRIVATE_LOGFMT_FIELDS_6(A,B,C,D,E,F)       , UE_PRIVATE_LOGFMT_FIELD(A), UE_PRIVATE_LOGFMT_FIELD(B), UE_PRIVATE_LOGFMT_FIELD(C), UE_PRIVATE_LOGFMT_FIELD(D), UE_PRIVATE_LOGFMT_FIELD(E), UE_PRIVATE_LOGFMT_FIELD(F)
#define UE_PRIVATE_LOGFMT_FIELDS_7(A,B,C,D,E,F,G)     , UE_PRIVATE_LOGFMT_FIELD(A), UE_PRIVATE_LOGFMT_FIELD(B), UE_PRIVATE_LOGFMT_FIELD(C), UE_PRIVATE_LOGFMT_FIELD(D), UE_PRIVATE_LOGFMT_FIELD(E), UE_PRIVATE_LOGFMT_FIELD(F), UE_PRIVATE_LOGFMT_FIELD(G)
#define UE_PRIVATE_LOGFMT_FIELDS_8(A,B,C,D,E,F,G,H)   , UE_PRIVATE_LOGFMT_FIELD(A), UE_PRIVATE_LOGFMT_FIELD(B), UE_PRIVATE_LOGFMT_FIELD(C), UE_PRIVATE_LOGFMT_FIELD(D), UE_PRIVATE_LOGFMT_FIELD(E), UE_PRIVATE_LOGFMT_FIELD(F), UE_PRIVATE_LOGFMT_FIELD(G), UE_PRIVATE_LOGFMT_FIELD(H)
#define UE_PRIVATE_LOGFMT_FIELDS_9(A,B,C,D,E,F,G,H,I) , UE_PRIVATE_LOGFMT_FIELD(A), UE_PRIVATE_LOGFMT_FIELD(B), UE_PRIVATE_LOGFMT_FIELD(C), UE_PRIVATE_LOGFMT_FIELD(D), UE_PRIVATE_LOGFMT_FIELD(E), UE_PRIVATE_LOGFMT_FIELD(F), UE_PRIVATE_LOGFMT_FIELD(G), UE_PRIVATE_LOGFMT_FIELD(H), UE_PRIVATE_LOGFMT_FIELD(I)

#define UE_PRIVATE_LOGFMT_FIELDS_10(A,B,C,D,E,F,G,H,I,J)             , UE_PRIVATE_LOGFMT_FIELD(A), UE_PRIVATE_LOGFMT_FIELD(B), UE_PRIVATE_LOGFMT_FIELD(C), UE_PRIVATE_LOGFMT_FIELD(D), UE_PRIVATE_LOGFMT_FIELD(E), UE_PRIVATE_LOGFMT_FIELD(F), UE_PRIVATE_LOGFMT_FIELD(G), UE_PRIVATE_LOGFMT_FIELD(H), UE_PRIVATE_LOGFMT_FIELD(I), UE_PRIVATE_LOGFMT_FIELD(J)
#define UE_PRIVATE_LOGFMT_FIELDS_11(A,B,C,D,E,F,G,H,I,J,K)           , UE_PRIVATE_LOGFMT_FIELD(A), UE_PRIVATE_LOGFMT_FIELD(B), UE_PRIVATE_LOGFMT_FIELD(C), UE_PRIVATE_LOGFMT_FIELD(D), UE_PRIVATE_LOGFMT_FIELD(E), UE_PRIVATE_LOGFMT_FIELD(F), UE_PRIVATE_LOGFMT_FIELD(G), UE_PRIVATE_LOGFMT_FIELD(H), UE_PRIVATE_LOGFMT_FIELD(I), UE_PRIVATE_LOGFMT_FIELD(J), UE_PRIVATE_LOGFMT_FIELD(K)
#define UE_PRIVATE_LOGFMT_FIELDS_12(A,B,C,D,E,F,G,H,I,J,K,L)         , UE_PRIVATE_LOGFMT_FIELD(A), UE_PRIVATE_LOGFMT_FIELD(B), UE_PRIVATE_LOGFMT_FIELD(C), UE_PRIVATE_LOGFMT_FIELD(D), UE_PRIVATE_LOGFMT_FIELD(E), UE_PRIVATE_LOGFMT_FIELD(F), UE_PRIVATE_LOGFMT_FIELD(G), UE_PRIVATE_LOGFMT_FIELD(H), UE_PRIVATE_LOGFMT_FIELD(I), UE_PRIVATE_LOGFMT_FIELD(J), UE_PRIVATE_LOGFMT_FIELD(K), UE_PRIVATE_LOGFMT_FIELD(L)
#define UE_PRIVATE_LOGFMT_FIELDS_13(A,B,C,D,E,F,G,H,I,J,K,L,M)       , UE_PRIVATE_LOGFMT_FIELD(A), UE_PRIVATE_LOGFMT_FIELD(B), UE_PRIVATE_LOGFMT_FIELD(C), UE_PRIVATE_LOGFMT_FIELD(D), UE_PRIVATE_LOGFMT_FIELD(E), UE_PRIVATE_LOGFMT_FIELD(F), UE_PRIVATE_LOGFMT_FIELD(G), UE_PRIVATE_LOGFMT_FIELD(H), UE_PRIVATE_LOGFMT_FIELD(I), UE_PRIVATE_LOGFMT_FIELD(J), UE_PRIVATE_LOGFMT_FIELD(K), UE_PRIVATE_LOGFMT_FIELD(L), UE_PRIVATE_LOGFMT_FIELD(M)
#define UE_PRIVATE_LOGFMT_FIELDS_14(A,B,C,D,E,F,G,H,I,J,K,L,M,N)     , UE_PRIVATE_LOGFMT_FIELD(A), UE_PRIVATE_LOGFMT_FIELD(B), UE_PRIVATE_LOGFMT_FIELD(C), UE_PRIVATE_LOGFMT_FIELD(D), UE_PRIVATE_LOGFMT_FIELD(E), UE_PRIVATE_LOGFMT_FIELD(F), UE_PRIVATE_LOGFMT_FIELD(G), UE_PRIVATE_LOGFMT_FIELD(H), UE_PRIVATE_LOGFMT_FIELD(I), UE_PRIVATE_LOGFMT_FIELD(J), UE_PRIVATE_LOGFMT_FIELD(K), UE_PRIVATE_LOGFMT_FIELD(L), UE_PRIVATE_LOGFMT_FIELD(M), UE_PRIVATE_LOGFMT_FIELD(N)
#define UE_PRIVATE_LOGFMT_FIELDS_15(A,B,C,D,E,F,G,H,I,J,K,L,M,N,O)   , UE_PRIVATE_LOGFMT_FIELD(A), UE_PRIVATE_LOGFMT_FIELD(B), UE_PRIVATE_LOGFMT_FIELD(C), UE_PRIVATE_LOGFMT_FIELD(D), UE_PRIVATE_LOGFMT_FIELD(E), UE_PRIVATE_LOGFMT_FIELD(F), UE_PRIVATE_LOGFMT_FIELD(G), UE_PRIVATE_LOGFMT_FIELD(H), UE_PRIVATE_LOGFMT_FIELD(I), UE_PRIVATE_LOGFMT_FIELD(J), UE_PRIVATE_LOGFMT_FIELD(K), UE_PRIVATE_LOGFMT_FIELD(L), UE_PRIVATE_LOGFMT_FIELD(M), UE_PRIVATE_LOGFMT_FIELD(N), UE_PRIVATE_LOGFMT_FIELD(O)
#define UE_PRIVATE_LOGFMT_FIELDS_16(A,B,C,D,E,F,G,H,I,J,K,L,M,N,O,P) , UE_PRIVATE_LOGFMT_FIELD(A), UE_PRIVATE_LOGFMT_FIELD(B), UE_PRIVATE_LOGFMT_FIELD(C), UE_PRIVATE_LOGFMT_FIELD(D), UE_PRIVATE_LOGFMT_FIELD(E), UE_PRIVATE_LOGFMT_FIELD(F), UE_PRIVATE_LOGFMT_FIELD(G), UE_PRIVATE_LOGFMT_FIELD(H), UE_PRIVATE_LOGFMT_FIELD(I), UE_PRIVATE_LOGFMT_FIELD(J), UE_PRIVATE_LOGFMT_FIELD(K), UE_PRIVATE_LOGFMT_FIELD(L), UE_PRIVATE_LOGFMT_FIELD(M), UE_PRIVATE_LOGFMT_FIELD(N), UE_PRIVATE_LOGFMT_FIELD(O), UE_PRIVATE_LOGFMT_FIELD(P)

#define UE_PRIVATE_LOGFMT_COUNT(...) UE_PRIVATE_LOGFMT_CALL(UE_PRIVATE_LOGFMT_COUNT_IMPL, (_, ##__VA_ARGS__, 16,15,14,13,12,11,10,9,8,7,6,5,4,3,2,1,0))
#define UE_PRIVATE_LOGFMT_COUNT_IMPL(_, A,B,C,D,E,F,G,H,I,J,K,L,M,N,O,P, Count, ...) Count

#define UE_PRIVATE_LOGFMT_CALL(F, A) UE_PRIVATE_LOGFMT_EXPAND(F A)
#define UE_PRIVATE_LOGFMT_EXPAND(X) X

#undef UE_API
