// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "Internationalization/Text.h"
#include "Templates/FunctionWithContext.h"
#include "Templates/UniquePtr.h"

#define UE_API CORE_API

class FCbFieldViewIterator;
class FCbWriter;

namespace UE { class FLogTemplate; }
namespace UE { struct FLogTemplateOptions; }
namespace UE::Logging::Private { struct FLogField; }

namespace UE::Logging::Private
{

/** Store a log template in an inline byte array. */
class FInlineLogTemplateStorage
{
public:
	inline void* Allocate(int32 Size)
	{
		Data.SetNum(Size);
		return Data.GetData();
	}

	inline const void* Get() const
	{
		return Data.GetData();
	}

private:
	// FLogTemplate is 8 bytes plus the encoded ops.
	// There are 2 ops for each field. There is 1 op for each contiguous region of literal text.
	// There is 1 op for each escaped character. There is 1 op to mark the end.
	// Most ops encode as 1 byte. A format string that has 12 fields with text surrounding each
	// field will typically be encoded in 46 bytes.
	TArray<uint8, TInlineAllocator<48>> Data;
};

/** Store a log template in a movable heap-allocated byte array. */
class FUniqueLogTemplateStorage
{
public:
	inline void* Allocate(int32 Size)
	{
		Data = MakeUnique<uint8[]>(Size);
		return Data.Get();
	}

	inline const void* Get() const
	{
		return Data.Get();
	}

private:
	TUniquePtr<uint8[]> Data;
};

UE_API void CreateLogTemplate(const TCHAR* Format, const FLogTemplateOptions& Options, const FLogField* Fields, int32 FieldCount, TFunctionWithContext<void* (int32)> Allocate);
UE_API void CreateLogTemplate(const UTF8CHAR* Format, const FLogTemplateOptions& Options, const FLogField* Fields, int32 FieldCount, TFunctionWithContext<void* (int32)> Allocate);
UE_API void CreateLocalizedLogTemplate(const FText& Format, const FLogTemplateOptions& Options, const FLogField* Fields, int32 FieldCount, TFunctionWithContext<void* (int32)> Allocate);
UE_API void CreateLocalizedLogTemplate(const TCHAR* TextNamespace, const TCHAR* TextKey, const TCHAR* Format, const FLogTemplateOptions& Options, const FLogField* Fields, int32 FieldCount, TFunctionWithContext<void* (int32)> Allocate);
UE_API void CreateLocalizedLogTemplate(const TCHAR* TextNamespace, const TCHAR* TextKey, const UTF8CHAR* Format, const FLogTemplateOptions& Options, const FLogField* Fields, int32 FieldCount, TFunctionWithContext<void* (int32)> Allocate);

UE_API void DestroyLogTemplate(FLogTemplate* Template);

} // UE::Logging::Private

namespace UE
{

/** Options to control how log templates are parsed and perform formatting. */
struct FLogTemplateOptions
{
	/**
	 * If true, allow field references at a depth greater than 1.
	 *
	 * Access objects within objects using a path of the form A/B/C.
	 * Access items within arrays using a path of the form A[0]/B/C[1].
	 */
	bool bAllowSubObjectReferences = false;
};

UE_API void FormatLogTo(FUtf8StringBuilderBase& Out, const FLogTemplate* Template, const FCbFieldViewIterator& Fields);
UE_API void FormatLogTo(FWideStringBuilderBase& Out, const FLogTemplate* Template, const FCbFieldViewIterator& Fields);
UE_API FText FormatLogToText(const FLogTemplate* Template, const FCbFieldViewIterator& Fields);

/** A log template that is templated on its storage. Use FInlineLogTemplate or FSharedLogTemplate. */
template <typename StorageType>
class TLogTemplate
{
public:
	inline explicit TLogTemplate(const TCHAR* Format, const FLogTemplateOptions& Options = {}, const Logging::Private::FLogField* Fields = nullptr, int32 FieldCount = 0)
	{
		Logging::Private::CreateLogTemplate(Format, Options, Fields, FieldCount, [this](int32 Size) { return Storage.Allocate(Size); });
	}
public:
	inline explicit TLogTemplate(const UTF8CHAR* Format, const FLogTemplateOptions& Options = {}, const Logging::Private::FLogField* Fields = nullptr, int32 FieldCount = 0)
	{
		Logging::Private::CreateLogTemplate(Format, Options, Fields, FieldCount, [this](int32 Size) { return Storage.Allocate(Size); });
	}

	inline explicit TLogTemplate(const FText& Format, const FLogTemplateOptions& Options = {}, const Logging::Private::FLogField* Fields = nullptr, int32 FieldCount = 0)
	{
		Logging::Private::CreateLocalizedLogTemplate(Format, Options, Fields, FieldCount, [this](int32 Size) { return Storage.Allocate(Size); });
	}

	inline explicit TLogTemplate(const TCHAR* TextNamespace, const TCHAR* TextKey, const TCHAR* Format, const FLogTemplateOptions& Options = {}, const Logging::Private::FLogField* Fields = nullptr, int32 FieldCount = 0)
	{
		Logging::Private::CreateLocalizedLogTemplate(TextNamespace, TextKey, Format, Options, Fields, FieldCount, [this](int32 Size) { return Storage.Allocate(Size); });
	}

	inline explicit TLogTemplate(const TCHAR* TextNamespace, const TCHAR* TextKey, const UTF8CHAR* Format, const FLogTemplateOptions& Options = {}, const Logging::Private::FLogField* Fields = nullptr, int32 FieldCount = 0)
	{
		Logging::Private::CreateLocalizedLogTemplate(TextNamespace, TextKey, Format, Options, Fields, FieldCount, [this](int32 Size) { return Storage.Allocate(Size); });
	}

	inline void FormatTo(FUtf8StringBuilderBase& Out, const FCbFieldViewIterator& Fields)
	{
		FormatLogTo(Out, Get(), Fields);
	}

	inline void FormatTo(FWideStringBuilderBase& Out, const FCbFieldViewIterator& Fields)
	{
		FormatLogTo(Out, Get(), Fields);
	}

	inline FText FormatToText(const FCbFieldViewIterator& Fields)
	{
		return FormatLogToText(Get(), Fields);
	}

	inline ~TLogTemplate()
	{
		Logging::Private::DestroyLogTemplate(Get());
	}

	inline FLogTemplate* Get() const
	{
		return (FLogTemplate*)Storage.Get();
	}

	inline FLogTemplate* Detach()
		requires requires (StorageType& S) { S.Detach(); }
	{
		return (FLogTemplate*)Storage.Detach();
	}

private:
	TLogTemplate() = default;

	StorageType Storage;
};

/** A log template that stores its data inline. Best choice for a stack-based temporary. */
using FInlineLogTemplate = TLogTemplate<Logging::Private::FInlineLogTemplateStorage>;

/** A log template that stores its data on the heap. A reasonable default when for non-temporary templates. */
using FUniqueLogTemplate = TLogTemplate<Logging::Private::FUniqueLogTemplateStorage>;

/**
 * Serializes a localized log format template to compact binary. Call from SerializeForLog.
 *
 * @param Writer   A writer to serialize $locformat, $locns, $lockey fields to.
 * @param Format   A localized format template to serialize into the writer. Must have a namespace and key!
 */
UE_API void SerializeLogFormat(FCbWriter& Writer, const FText& Format);

} // UE

#undef UE_API
