// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

/** The severity of the message type */
namespace EMessageSeverity
{
	/** Ordered according to their severity */
	enum Type : int
	{
		CriticalError UE_DEPRECATED(5.1, "CriticalError was removed because it can't trigger an assert at the callsite. Use 'checkf' instead.") = 0,
		Error = 1,
		PerformanceWarning = 2,
		Warning = 3,
		Info = 4,	// Should be last
	};
}

/** Delgate used when clocking a message token */
DECLARE_DELEGATE_OneParam(FOnMessageTokenActivated, const TSharedRef<class IMessageToken>&);

namespace EMessageToken
{
	enum Type
	{
		Action,
		Actor,
		AssetName,
		AssetData,
		Documentation,
		Image,
		Object,
		Severity,
		Text,
		Tutorial,
		URL,
		EdGraph,
		DynamicText,
		Fix,
	};
}

/** A single message token for a FTokenizedMessage instance */
class IMessageToken : public TSharedFromThis<IMessageToken>
{
public:

	/**
	 * Virtual destructor
	 */
	virtual ~IMessageToken() = default;

	/** 
	 * Get the type of this message token
	 * 
	 * @returns the type of the token
	 */
	virtual EMessageToken::Type GetType() const = 0;

	/** 
	 * Get a string representation of this token
	 * 
	 * @returns a string representing this token
	 */
	virtual const FText& ToText() const
	{
		return CachedText;
	}

	/** 
	 * Get the activated delegate associated with this token, if any
	 * 
	 * @returns a reference to the delegate
	 */
	virtual const FOnMessageTokenActivated& GetOnMessageTokenActivated() const
	{
		return MessageTokenActivated;
	}

	/** 
	 * Set the activated delegate associated with this token
	 * 
	 * @returns a reference to this token, for chaining
	 */
	virtual TSharedRef<IMessageToken> OnMessageTokenActivated( FOnMessageTokenActivated InMessageTokenActivated )
	{
		MessageTokenActivated = InMessageTokenActivated;
		return AsShared();
	}

protected:
	/** A delegate for when this token is activated (e.g. hyperlink clicked) */
	FOnMessageTokenActivated MessageTokenActivated;

	/** Cached string representation of this token */
	FText CachedText;
};

/** This class represents a rich tokenized message, such as would be used for compiler output with 'hyperlinks' to source file locations */
class FTokenizedMessage : public TSharedFromThis<FTokenizedMessage>
{
	// The private token allows only members or friends to call MakeShared.
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:

	/** 
	 * Creates a new FTokenizedMessage
	 * 
	 * @param	InSeverity			The severity of the message.
	 * @param	InMessageString		The string to display for this message. If this is not empty, then a string token will be added to this message.
	 * @returns	the generated message for temporary storage or so other tokens can be added to it if required.
	 */
	CORE_API static TSharedRef<FTokenizedMessage> Create(EMessageSeverity::Type InSeverity, const FText& InMessageText = FText());

	/**
	 * Clone this message.
	 * @note The message tokens are shared between the original and the clone.
	 */
	CORE_API TSharedRef<FTokenizedMessage> Clone() const;

	/** 
	 * Get this tokenized message as a string
	 * 
	 * @returns a string representation of this message
	 */
	CORE_API FText ToText() const;
	
	/** 
	 * Adds a token to a message.
	 * @param	InMessage	The message to insert a token into
	 * @param	InToken		The token to insert
	 * @returns this message, for chaining calls.
	 */
	CORE_API TSharedRef<FTokenizedMessage> AddToken( const TSharedRef<IMessageToken>& InToken );

	/** 
	 * Adds a text token to a message.
	 * @param	InMessage	The message to insert a token into
	 * @param	InText		The text to insert as a token
	 * @returns this message, for chaining calls.
	 */
	CORE_API TSharedRef<FTokenizedMessage> AddText(const FText& InText);

	/** 
	 * Adds a text token to a message as by calling FText::FormatOrdered
	 * @param	InMessage	The message to insert a token into
	 * @param	InText		The text to insert as a token
	 * @returns this message, for chaining calls.
	 */
	template<typename... TArguments>
	TSharedRef<FTokenizedMessage> AddText(FTextFormat InTextFormat, TArguments&&... InArgs)
	{
		return AddText(FText::FormatOrdered(InTextFormat, Forward<TArguments...>(InArgs...)));
	}

	/** 
	 * Sets the severity of this message
	 * 
	 * @param	InSeverity	The severity to set this message to

	 */
	CORE_API void SetSeverity( const EMessageSeverity::Type InSeverity );

	/** 
	 * Gets the severity of this message
	 * 
	 * @returns the severity of this message
	 */
	CORE_API EMessageSeverity::Type GetSeverity() const;

	/**
	 * Get the tokens in this message
	 * 
	 * @returns an array of tokens
	 */
	CORE_API const TArray< TSharedRef<IMessageToken> >& GetMessageTokens() const;

	/**
	 * Sets up a token action for the message as a whole (not to be displayed...
	 * intended to be invoked from a double click).
	 * 
	 * @param  InToken	A token for the entire message to link to.
	 */
	CORE_API void SetMessageLink(const TSharedRef<IMessageToken>& InToken);

	/**
	 * Gets the token action associated with this messages as a whole. Should
	 * link to an associated element.
	 * 
	 * @return A token action that was set via SetMessageLink().
	 */
	CORE_API TSharedPtr<IMessageToken> GetMessageLink() const;

	/** 
	 * Helper function for getting a severity as text
	 *
	 * @param	InSeverity	the severity to use
	 * @returns a string representation of the severity
	 */
	CORE_API static FText GetSeverityText(EMessageSeverity::Type InSeverity);

	/** 
	 * Helper function for getting a severity as an icon name
	 *
	 * @param	InSeverity	the severity to use
	 * @returns the name of the icon for the severity
	 */
	CORE_API static FName GetSeverityIconName(EMessageSeverity::Type InSeverity);
	
	/** @returns Identifier for the message, if set, else NAME_None */
	CORE_API FName GetIdentifier() const;
	
	/** Assigns Identifier for the message to the provided name */
	CORE_API void SetIdentifier(FName InIdentifier);
	
	/** @returns Indentation level for the message, defaults to 0 */
	CORE_API int32 GetIndentationLevel() const;

	/** Sets Indentation level for the message, used for padding message within MessageLog widget */
	CORE_API void SetIndentationLevel(int32 Level);

	/** Private constructor - we want to only create these structures as shared references via Create() */
	explicit FTokenizedMessage(FPrivateToken)
		: Severity( EMessageSeverity::Info )
		, Identifier(NAME_None)
	{
	}

protected:

	/** the array of message tokens this message contains */
	TArray< TSharedRef<IMessageToken> > MessageTokens;

	/** A token associated with the entire message (doesn't display) */
	TSharedPtr<IMessageToken> MessageLink;

private:

	/** The severity of this message */
	EMessageSeverity::Type Severity;

	/** Identifier for the message */
	FName Identifier;
	
	/** Requested indentation level for this message, used for padding message visually */
	int32 IndentationLevel = 0;
};

/** Basic message token with a localized text payload */
class FTextToken : public IMessageToken
{
	// The private token allows only members or friends to call MakeShared.
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	/** Factory method, tokens can only be constructed as shared refs */
	CORE_API static TSharedRef<FTextToken> Create(const FText& InMessage, bool InIsSourceLinkOnLeft = true);

	/** Begin IMessageToken interface */
	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::Text;
	}
	/** End IMessageToken interface */

	/** Private constructor */
	FTextToken(FPrivateToken, const FText& InMessage, bool InIsSourceLinkOnLeft)
	{
		CachedText = InMessage;
		bIsSourceLinkOnLeft = InIsSourceLinkOnLeft;
	}

	bool IsSourceLinkOnLeft() const
	{
		return bIsSourceLinkOnLeft;
	}

private:
	/** Whether the source address is located on the left or right of the text message. */
	bool bIsSourceLinkOnLeft;
};

/** Message token with a localized attribute text payload */
class FDynamicTextToken : public IMessageToken
{
	// The private token allows only members or friends to call MakeShared.
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	/** Factory method, tokens can only be constructed as shared refs */
	CORE_API static TSharedRef<FDynamicTextToken> Create(const TAttribute<FText>& InMessage);

	/** Begin IMessageToken interface */
	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::DynamicText;
	}

	virtual const FText& ToText() const
	{
		return Message.Get();
	}
	/** End IMessageToken interface */

	const TAttribute<FText>& GetTextAttribute() const
	{
		return Message;
	}

	/** Private constructor */
	FDynamicTextToken(FPrivateToken, const TAttribute<FText>& InMessage)
		: Message(InMessage)
	{
		CachedText = InMessage.Get();
	}

private:
	/** The attribute text of this message */
	TAttribute<FText> Message;
};

/** Basic message token with an icon/image payload */
class FImageToken : public IMessageToken
{
	// The private token allows only members or friends to call MakeShared.
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	/** Factory method, tokens can only be constructed as shared refs */
	CORE_API static TSharedRef<FImageToken> Create(const FName& InImageName);

	/** Begin IMessageToken interface */
	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::Image;
	}
	/** End IMessageToken interface */

	/** Get the name of the image for this token */
	const FName& GetImageName() const
	{
		return ImageName;
	}

	/** Private constructor */
	FImageToken(FPrivateToken, const FName& InImageName)
		: ImageName(InImageName)
	{
		CachedText = FText::FromName( InImageName );
	}

private:
	/** A name to be used as a brush in this message */
	FName ImageName;
};

/** Basic message token with a severity payload */
class FSeverityToken : public IMessageToken
{
	// The private token allows only members or friends to call MakeShared.
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	/** Factory method, tokens can only be constructed as shared refs */
	CORE_API static TSharedRef<FSeverityToken> Create(EMessageSeverity::Type InSeverity);

	/** Begin IMessageToken interface */
	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::Severity;
	}
	/** End IMessageToken interface */

	/** Get the severity of this token */
	EMessageSeverity::Type GetSeverity() const
	{
		return Severity;
	}

	/** Private constructor */
	FSeverityToken(FPrivateToken, EMessageSeverity::Type InSeverity)
		: Severity(InSeverity)
	{
		CachedText = FTokenizedMessage::GetSeverityText( InSeverity );
	}

private:
	/** A severity for this token */
	EMessageSeverity::Type Severity;
};

/** Basic message token that defaults is activated method to traverse a URL */
class FURLToken : public IMessageToken
{
	// The private token allows only members or friends to call MakeShared.
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	/** Factory method, tokens can only be constructed as shared refs */
	CORE_API static TSharedRef<FURLToken> Create(const FString& InURL, const FText& InMessage = FText());

	/** Begin IMessageToken interface */
	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::URL;
	}
	/** End IMessageToken interface */

	/** Get the URL used by this token */
	const FString& GetURL() const
	{
		return URL;
	}

	/** Private constructor */
	inline FURLToken(FPrivateToken, const FString& InURL, const FText& InMessage)
		: FURLToken(InURL, InMessage)
	{
	}

private:
	/** Private constructor */
	FURLToken(const FString& InURL, const FText& InMessage);

	/**
	 * Delegate used to visit a URL
	 * @param	Token		The token that was clicked
	 * @param	InURL		The URL to visit
	 */
	static void VisitURL(const TSharedRef<IMessageToken>& Token, FString InURL);

	/** The URL we will follow */
	FString URL;
};

/** 
 * Basic message token that defaults its activated method to find a file
 * Intended to hook into things like the content browser.
 */
class FAssetNameToken : public IMessageToken
{
	// The private token allows only members or friends to call MakeShared.
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	/** Factory method, tokens can only be constructed as shared refs */
	CORE_API static TSharedRef<FAssetNameToken> Create(const FString& InAssetName, const FText& InMessage = FText());

	/** Begin IMessageToken interface */
	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::AssetName;
	}
	/** End IMessageToken interface */

	/** Get the filename used by this token */
	const FString& GetAssetName() const
	{
		return AssetName;
	}

	DECLARE_DELEGATE_OneParam(FOnGotoAsset, const FString&);
	static FOnGotoAsset& OnGotoAsset()
	{
		return GotoAsset;
	}

	/** Private constructor */
	inline FAssetNameToken(FPrivateToken, const FString& InAssetName, const FText& InMessage)
		: FAssetNameToken(InAssetName, InMessage)
	{
	}

private:
	/** Private constructor */
	FAssetNameToken(const FString& InAssetName, const FText& InMessage);

	/**
	 * Delegate used to find a file
	 * @param	Token		The token that was clicked
	 * @param	InURL		The file to find
	 */
	static void FindAsset(const TSharedRef<IMessageToken>& Token, FString InAssetName);

	/** The asset name we will find */
	FString AssetName;

	/** The delegate we will use to go to our file */
	CORE_API static FOnGotoAsset GotoAsset;
};

/** 
 * Basic message token that defaults is activated method to access UDN documentation.
 */
class FDocumentationToken : public IMessageToken
{
	// The private token allows only members or friends to call MakeShared.
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	/** Factory method, tokens can only be constructed as shared refs */
	CORE_API static TSharedRef<FDocumentationToken> Create(const FString& InDocumentationLink, const FString& InPreviewExcerptLink = FString(), const FString& InPreviewExcerptName = FString());

	/** Begin IMessageToken interface */
	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::Documentation;
	}
	/** End IMessageToken interface */

	/** Get the documentation link used by this token */
	const FString& GetDocumentationLink() const
	{
		return DocumentationLink;
	}

	/** Get the documentation excerpt link used by this token */
	const FString& GetPreviewExcerptLink() const
	{
		return PreviewExcerptLink;
	}

	/** Get the documentation excerpt name used by this token */
	const FString& GetPreviewExcerptName() const
	{
		return PreviewExcerptName;
	}

	inline FDocumentationToken(FPrivateToken, const FString& InDocumentationLink, const FString& InPreviewExcerptLink, const FString& InPreviewExcerptName)
		: FDocumentationToken(InDocumentationLink, InPreviewExcerptLink, InPreviewExcerptName)
	{
	}

protected:
	/** Protected constructor */
	FDocumentationToken(const FString& InDocumentationLink, const FString& InPreviewExcerptLink, const FString& InPreviewExcerptName);

private:
	/** The documentation path we link to when clicked */
	FString DocumentationLink;

	/** The link we display an excerpt from */
	FString PreviewExcerptLink;

	/** The excerpt to display */
	FString PreviewExcerptName;
};


DECLARE_DELEGATE(FOnActionTokenExecuted);
DECLARE_DELEGATE_RetVal(bool, FCanExecuteActionToken);

/**
 * Message token that performs an action when activated.
 */
class FActionToken
	: public IMessageToken
{
	// The private token allows only members or friends to call MakeShared.
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:

	/** Factory methods, tokens can only be constructed as shared refs */
	CORE_API static TSharedRef<FActionToken> Create(const FText& InActionName, const FText& InActionDescription, const FOnActionTokenExecuted& InAction, bool bInSingleUse = false);
	CORE_API static TSharedRef<FActionToken> Create(const FText& InActionName, const FText& InActionDescription, const FOnActionTokenExecuted& InAction, const FCanExecuteActionToken& InCanExecuteAction, bool bInSingleUse = false);

	/** Executes the assigned action delegate. */
	void ExecuteAction()
	{
		ActionDelegate.ExecuteIfBound();
		bActionExecuted = true;
	}

	/** Gets the action's description text. */
	const FText& GetActionDescription()
	{
		return ActionDescription;
	}

	/** Returns true if the action can be activated */
	bool CanExecuteAction() const
	{
		return ActionDelegate.IsBound() && (!bSingleUse || !bActionExecuted) && (!CanExecuteActionDelegate.IsBound() || CanExecuteActionDelegate.Execute());
	}

	/** Returns true if the action is properly set */
	bool IsValidAction() const
	{
		return ActionDelegate.IsBound();
	}

	// IMessageToken interface
	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::Action;
	}

	inline FActionToken(FPrivateToken, const FText& InActionName, const FText& InActionDescription, const FOnActionTokenExecuted& InAction, bool bInSingleUse)
		: FActionToken(InActionName, InActionDescription, InAction, bInSingleUse)
	{
	}

	inline FActionToken(FPrivateToken, const FText& InActionName, const FText& InActionDescription, const FOnActionTokenExecuted& InAction, const FCanExecuteActionToken& InCanExecuteAction, bool bInSingleUse)
		: FActionToken(InActionName, InActionDescription, InAction, InCanExecuteAction, bInSingleUse)
	{
	}

protected:

	/** Hidden constructors. */
	FActionToken(const FText& InActionName, const FText& InActionDescription, const FOnActionTokenExecuted& InAction, bool bInSingleUse)
		: ActionDelegate(InAction)
		, ActionDescription(InActionDescription)
		, bSingleUse(bInSingleUse)
		, bActionExecuted(false)
	{
		CachedText = InActionName;
	}

	FActionToken(const FText& InActionName, const FText& InActionDescription, const FOnActionTokenExecuted& InAction, const FCanExecuteActionToken& InCanExecuteAction, bool bInSingleUse)
		: ActionDelegate(InAction)
		, CanExecuteActionDelegate(InCanExecuteAction)
		, ActionDescription(InActionDescription)
		, bSingleUse(bInSingleUse)
		, bActionExecuted(false)
	{
		CachedText = InActionName;
	}

private:

	/** Holds a delegate that is executed when this token is activated. */
	FOnActionTokenExecuted ActionDelegate;

	/** Holds a delegate that is executed to know whether this token's action can actually execute. */
	FCanExecuteActionToken CanExecuteActionDelegate;

	/** The action's description text. */
	const FText ActionDescription;
	
	/** If true, the action can only be performed once. */
	bool bSingleUse;

	/** If true, the action has been executed already. */
	bool bActionExecuted;
};


class FTutorialToken
	: public IMessageToken
{
	// The private token allows only members or friends to call MakeShared.
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:

	/** Factory method, tokens can only be constructed as shared refs */
	CORE_API static TSharedRef<FTutorialToken> Create(const FString& TutorialAssetName);

public:

	// IMessageToken interface

	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::Tutorial;
	}

	/** Get the tutorial asset name stored in this token. */
	const FString& GetTutorialAssetName() const
	{
		return TutorialAssetName;
	}

	FTutorialToken(FPrivateToken, const FString& InTutorialAssetName)
		: FTutorialToken(InTutorialAssetName)
	{
	}

protected:
	/** Protected constructor */
	FTutorialToken( const FString& InTutorialAssetName )
		: TutorialAssetName(InTutorialAssetName)
	{ }

private:

	/** The name of the tutorial asset. */
	FString TutorialAssetName;
};

/** 
 * Basic message token that defaults its activated method to select an actor in the opened level
 */
class FActorToken : public IMessageToken
{
	// The private token allows only members or friends to call MakeShared.
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	/** Factory method, tokens can only be constructed as shared refs */
	CORE_API static TSharedRef<FActorToken> Create(const FString& InActorPath, const FGuid& InActorGuid, const FText& InMessage = FText());

	/** Begin IMessageToken interface */
	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::Actor;
	}

	virtual const FOnMessageTokenActivated& GetOnMessageTokenActivated() const override;
	/** End IMessageToken interface */

	/** Get the actor name used by this token */
	const FString& GetActorPath() const
	{
		return ActorPath;
	}

	/** Get the actor guid used by this token */
	const FGuid& GetActorGuid() const
	{
		return ActorGuid;
	}

	/** Get the delegate for default token activation */
	static FOnMessageTokenActivated& DefaultOnMessageTokenActivated()
	{
		return DefaultMessageTokenActivated;
	}

	/** Private constructor */
	inline FActorToken(FPrivateToken, const FString& InActorPath, const FGuid& InActorGuid, const FText& InMessage)
		: FActorToken(InActorPath, InActorGuid, InMessage)
	{
	}

private:
	/** Private constructor */
	FActorToken(const FString& InActorPath, const FGuid& InActorGuid, const FText& InMessage);

	/** The actor path we will select */
	FString ActorPath;

	/** The actor guid we will select */
	FGuid ActorGuid;

	/** The default activation method, if any */
	CORE_API static FOnMessageTokenActivated DefaultMessageTokenActivated;
};

namespace UE::DataValidation
{
struct IFixer;
}

struct FFixResult
{
	bool bIsSuccess = true;
	FText Message;

	static FFixResult Success(const FText& Message)
	{
		return {true, Message};
	}

	static FFixResult Success()
	{
		return {true, FText::GetEmpty()};
	}

	static FFixResult Failure(const FText& Message)
	{
		return {false, Message};
	}

	// NOTE: No overload for Failure that accepts no message, because in case of an error you
	// should *always* let the user know what went wrong.
};

enum class EFixApplicability : uint8 
{
	/** Fix hasn't been applied yet but can be applied */
	CanBeApplied,
	/** Fix has been applied successfully */
	Applied,
	/** Fix was not applied by the user, and instead a different fix was chosen */
	DidNotApply,
};

/** A fix action token. These are displayed in a separate list below the log message. */
class FFixToken : public IMessageToken
{
	// The private token allows only members or friends to call MakeShared.
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	/** Creates an FFixToken out of an IFixer and a fix index. Prefer IFixer::CreateToken over this. */
	CORE_API static TSharedRef<FFixToken> Create(const FText& InName, TSharedRef<UE::DataValidation::IFixer> InFixer, int32 InFixIndex);

	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::Fix;
	}

	TSharedRef<UE::DataValidation::IFixer> GetFixer() const { return Fixer.ToSharedRef(); }
	int32 GetFixIndex() const { return FixIndex; }
	void SetFix(TSharedRef<UE::DataValidation::IFixer> InFixer, int32 InFixIndex)
	{
		Fixer = InFixer;
		FixIndex = InFixIndex;
	}

	FFixToken(FPrivateToken, const FText& InName, TSharedPtr<UE::DataValidation::IFixer> Fixer, int32 FixIndex)
		: FFixToken(InName, MoveTemp(Fixer), FixIndex)
	{}

private:
	CORE_API FFixToken(const FText& InName, TSharedPtr<UE::DataValidation::IFixer> InFixer, int32 InFixIndex);

	TSharedPtr<UE::DataValidation::IFixer> Fixer;
	int32 FixIndex = INDEX_NONE;
};
