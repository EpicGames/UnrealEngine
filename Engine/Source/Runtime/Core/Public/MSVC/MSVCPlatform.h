// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	MSVCPlatform.h: Setup for any MSVC-using platform
==================================================================================*/

#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.7, "Use Microsoft/MSVCPlatform.h instead.")

#if PLATFORM_MICROSOFT
#include "Microsoft/MSVCPlatform.h"
#endif
