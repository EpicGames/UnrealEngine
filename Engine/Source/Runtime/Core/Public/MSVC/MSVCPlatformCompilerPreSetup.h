// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Cause circular dependencies for header units

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.7, "Use Microsoft/MSVCPlatformCompilerPreSetup.h instead.")

#if PLATFORM_MICROSOFT
#include "Microsoft/MSVCPlatformCompilerPreSetup.h"
#endif
