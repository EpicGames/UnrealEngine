// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Mac/MacSystemIncludes.h"

#define MAC_SEPARATE_GAME_THREAD 1 // Separate the main & game threads so that we better handle the interaction between the Cocoa's event delegates and UE's event polling.

/* Custom run-loop modes for Unreal that process only certain kinds of events to simulate Windows event ordering. */
CORE_API extern NSString* UnrealNilEventMode; /* Process only mandatory events */
CORE_API extern NSString* UnrealShowEventMode; /* Process only show window events */
CORE_API extern NSString* UnrealResizeEventMode; /* Process only resize/move window events */
CORE_API extern NSString* UnrealFullscreenEventMode; /* Process only fullscreen mode events */
CORE_API extern NSString* UnrealCloseEventMode; /* Process only close window events */
CORE_API extern NSString* UnrealIMEEventMode; /* Process only input method events */

#if MAC_SEPARATE_GAME_THREAD
/** Thread ID of the Mac MainThread */
extern CORE_API uint32 GMacMainThreadId;
#endif

@interface NSThread (FCocoaThread)
+ (NSThread*) gameThread; // Returns the main game thread, or nil if has yet to be constructed.
+ (bool) isGameThread; // True if the current thread is the main game thread, else false.
- (bool) isGameThread; // True if this thread object is the main game thread, else false.
@end

@interface FCocoaGameThread : NSThread
- (id)init; // Override that sets the variable backing +[NSThread gameThread], do not override in a subclass.
- (id)initWithTarget:(id)Target selector:(SEL)Selector object:(id)Argument; // Override that sets the variable backing +[NSThread gameThread], do not override in a subclass.
- (void)dealloc; // Override that clears the variable backing +[NSThread gameThread], do not override in a subclass.
- (void)main; // Override that sets the variable backing +[NSRunLoop gameRunLoop], do not override in a subclass.
@end

/**
 * Schedule a block to be executed on MainThread.
 * @param Block - The block to execute on MainThread, example: ^{ UE_LOG(LogMac, Log, TEXT("Hello MainThread"); }
 * @param bWait - Wether or not to wait until the block is executed.
 * @param WaitMode - The mode we are allowed to execute while waiting. (Only use when bWait = true)
 * 
 * Avoid being too restrictive on the WaitMode or you could cause a deadlock by preventing further progress.
 */
CORE_API void MainThreadCall(dispatch_block_t Block, bool const bWait = true, NSString* WaitMode = NSDefaultRunLoopMode);

UE_DEPRECATED(5.6, "Use the alternative with inverted last parameters")
CORE_API inline void MainThreadCall(dispatch_block_t Block, NSString* WaitMode, bool const bWait = true)
{
	MainThreadCall(Block, bWait, WaitMode);
}

/**
 * Schedule a block to be executed on MainThread with a return value.
 * @param Block - The block to execute on MainThread, example: ^{ return 5; }
 * @param WaitMode - The mode we are allowed to execute while waiting.
 * 
 * Avoid being too restrictive on the WaitMode or you could cause a deadlock by preventing further progress.
 */
template<typename ReturnType>
ReturnType MainThreadReturn(ReturnType (^Block)(void), NSString* WaitMode = NSDefaultRunLoopMode)
{
	__block ReturnType ReturnValue;
	MainThreadCall(^{ ReturnValue = Block(); }, true, WaitMode);
	return ReturnValue;
}

/**
 * Schedule a block to be executed on GameThread
 * @param Block - The block to execute on GameThread, example: ^{ UE_LOG(LogMac, Log, TEXT("Hello GameThread"); }
 * @param bWait - Wether or not to wait until the block is executed.
 * @param SendModes - The modes for the block we are scheduling, this is use to specify who can execute the scheduled block.
 * 
 * Be careful to make sure that GameThread will be processing at least one of the modes or you could end up causing a deadlock by preventing further progress.
 */
CORE_API void GameThreadCall(dispatch_block_t Block, bool const bWait = true, NSArray* SendModes = @[ NSDefaultRunLoopMode ]);

UE_DEPRECATED(5.6, "Use the alternative with inverted last parameters")
CORE_API inline void GameThreadCall(dispatch_block_t Block, NSArray* SendModes, bool const bWait = true)
{
	GameThreadCall(Block, bWait, SendModes);
}

/**
 * Schedule a block to be executed on GameThread with a return value.
 * @param Block - The block to execute on GameThread, example: ^{ return 5; }
 * @param SendModes - The modes for the block we are scheduling, this is use to specify who can execute the scheduled block.
 * 
 * Be careful to make sure that GameThread will be processing at least one of the modes or you could end up causing a deadlock by preventing further progress.
 */
template<typename ReturnType>
ReturnType GameThreadReturn(ReturnType (^Block)(void), NSArray* SendModes = @[ NSDefaultRunLoopMode ])
{
	__block ReturnType ReturnValue;
	GameThreadCall(^{ ReturnValue = Block(); }, true, SendModes);
	return ReturnValue;
}

CORE_API void RunGameThread(id Target, SEL Selector);

CORE_API void ProcessGameThreadEvents(void);
