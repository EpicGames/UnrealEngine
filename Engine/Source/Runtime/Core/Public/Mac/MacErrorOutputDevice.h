// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/OutputDeviceError.h"

class CORE_API FMacErrorOutputDevice : public FOutputDeviceError
{
public:
	/** Constructor, initializing member variables */
	FMacErrorOutputDevice();

	/**
	 * Serializes the passed in data unless the current event is suppressed.
	 *
	 * @param	Data	Text to log
	 * @param	Event	Event name used for suppression purposes
	 */
	virtual void Serialize( const TCHAR* Msg, ELogVerbosity::Type Verbosity, const class FName& Category ) override;

	/**
	 * Error handling function that is being called from within the system wide global
	 * error handler, e.g. using structured exception handling on the PC.
	 */
	virtual void HandleError() override;

private:
	int32		ErrorPos;

protected:
	/**
	 * Callback to allow FMacApplicationErrorOutputDevice to restore the UI.
	 */
	virtual void HandleErrorRestoreUI();
};
