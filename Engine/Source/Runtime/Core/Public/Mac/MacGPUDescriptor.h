// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	MacPlatformMiscEx.h: Mac platform additional misc functions that requires sdk depdendencies
==============================================================================================*/

#pragma once

#include "Containers/Map.h"
#include "Mac/MacSystemIncludes.h"

/** Common descriptor of each GPU in the OS that provides stock details about the GPU that are innaccessible from the higher-level rendering APIs and provides a direct link to the GPU in the IORegistry. */
template<typename T>
class FMacGPUDescriptorCommon
{
	public:
		virtual ~FMacGPUDescriptorCommon();

		FMacGPUDescriptorCommon& operator=(FMacGPUDescriptorCommon const& Other);

		TMap<FString, float> GetPerformanceStatistics() const;

	protected:
		FMacGPUDescriptorCommon() = default;
		FMacGPUDescriptorCommon(FMacGPUDescriptorCommon const& Other) = delete;
		FMacGPUDescriptorCommon(FMacGPUDescriptorCommon&& Other) = delete;

		void CopyFrom(FMacGPUDescriptorCommon const& Other);

	public:
		NSString*	GPUName			= nil;
		NSString*	GPUMetalBundle	= nil;
		NSString*	GPUOpenGLBundle	= nil;
		NSString*	GPUBundleID		= nil;
		uint32		GPUVendorId		= 0;
		uint32		GPUDeviceId		= 0;
		uint32		GPUMemoryMB		= 0;
		uint32		GPUIndex		= 0;

		bool GPUHeadless			= false;
};

#if PLATFORM_MAC_X86
	/** Intel architecture descriptor of each GPU in the OS that provides stock details about the GPU that are innaccessible from the higher-level rendering APIs and provides a direct link to the GPU in the IORegistry. */
	class FMacGPUDescriptor : public FMacGPUDescriptorCommon<FMacGPUDescriptor>
	{
		friend class FMacGPUDescriptorCommon<FMacGPUDescriptor>;

		public:
			FMacGPUDescriptor() = default;
			FMacGPUDescriptor(FMacGPUDescriptor const& Other);

			virtual ~FMacGPUDescriptor();

		protected:
			void CopyFromImpl(FMacGPUDescriptorCommon const& Other);
			TMap<FString, float> GetPerformanceStatisticsImpl() const;

		public:
			uint64 RegistryID;
			uint32 PCIDevice; // This is really an io_registry_entry_t which is a mach port name

	};

#elif PLATFORM_MAC_ARM64
	class FMacGPUDescriptor : public FMacGPUDescriptorCommon<FMacGPUDescriptor>
	{
		friend class FMacGPUDescriptorCommon<FMacGPUDescriptor>;

		public:
			FMacGPUDescriptor() = default;
			FMacGPUDescriptor(FMacGPUDescriptor const& Other);

			virtual ~FMacGPUDescriptor();

		protected:
			void CopyFromImpl(FMacGPUDescriptorCommon const& Other);
			TMap<FString, float> GetPerformanceStatisticsImpl() const;

		public:
			uint64 RegistryID = 0;

	};

#else
	#error "Undefined Mac platform"
#endif
