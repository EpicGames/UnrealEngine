// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/MemoryBase.h"

typedef struct _malloc_zone_t malloc_zone_t;
struct FMacCrashContext;

/*
 * A malloc interface using a unique malloc zone
 */
class FMacMallocZone : public FMalloc
{
public:
	FMacMallocZone( uint64 const InitialSize );
	virtual ~FMacMallocZone();
	
	// FMalloc interface.
	virtual void* Malloc( SIZE_T Size, uint32 Alignment ) override;
	
	virtual void* Realloc( void* Ptr, SIZE_T NewSize, uint32 Alignment ) override;
	
	virtual void Free( void* Ptr ) override;
	
	virtual bool GetAllocationSize(void *Original, SIZE_T &SizeOut) override;
	
	virtual bool IsInternallyThreadSafe() const override;
	
	virtual bool ValidateHeap() override;
	
	virtual const TCHAR* GetDescriptiveName() override;
	
protected:
	malloc_zone_t* MemoryZone;
};

/*
 * Specific FMacMallocZone for the crash handler malloc override so that we avoid malloc reentrancy problems
 */
class FMacMallocCrashHandler : public FMacMallocZone
{
public:
	FMacMallocCrashHandler( uint64 const InitialSize );
	
	virtual ~FMacMallocCrashHandler();
	
	void Enable( FMacCrashContext* Context, uint32 CrashedThreadId );
	
	// FMalloc interface.
	virtual void* Malloc( SIZE_T Size, uint32 Alignment ) override;
	
	virtual void* Realloc( void* Ptr, SIZE_T NewSize, uint32 Alignment ) override;
	
	virtual void Free( void* Ptr ) override;
	
	virtual bool GetAllocationSize( void *Original, SIZE_T &SizeOut ) override;
	
	virtual const TCHAR* GetDescriptiveName() override;

private:	
	bool IsOnCrashedThread(void);
	
private:
	FMalloc* OriginalHeap;
	FMacCrashContext* CrashContext;
	int32 ThreadId;
};
