// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Apple/ApplePlatform.h"

#define PLATFORM_MAC_USE_CHAR16 1 UE_DEPRECATED_MACRO(5.7, "PLATFORM_MAC_USE_CHAR16 has been deprecated and should be replaced with 1.")

/**
* Mac specific types
**/
struct FMacPlatformTypes : public FGenericPlatformTypes
{
	typedef unsigned int		DWORD;
	typedef size_t				SIZE_T;
	typedef decltype(NULL)		TYPE_OF_NULL;
	typedef char16_t			WIDECHAR;
	typedef WIDECHAR			TCHAR;
};

typedef FMacPlatformTypes FPlatformTypes;

// Define ARM64 / X86 here so we can run UBT once for both platforms
#if __is_target_arch(arm64) || __is_target_arch(arm64e)
#	define PLATFORM_MAC_ARM64							1
#	define PLATFORM_MAC_X86								0
#else
#	define PLATFORM_MAC_ARM64							0
#	define PLATFORM_MAC_X86								1
#endif

// Base defines, must define these for the platform, there are no defaults
#define PLATFORM_DESKTOP								1
#define PLATFORM_CAN_SUPPORT_EDITORONLY_DATA			1

// Base defines, defaults are commented out
//#define PLATFORM_EXCEPTIONS_DISABLED					!PLATFORM_DESKTOP
#define PLATFORM_ENABLE_VECTORINTRINSICS_NEON			PLATFORM_MAC_ARM64
// FMA3 support was added starting from Intel Haswell
#ifndef PLATFORM_ALWAYS_HAS_FMA3
#	define PLATFORM_ALWAYS_HAS_FMA3						0
#endif
//#define PLATFORM_USE_LS_SPEC_FOR_WIDECHAR				1
#define PLATFORM_COMPILER_DISTINGUISHES_INT_AND_LONG	1
#define PLATFORM_WCHAR_IS_4_BYTES						1
#define PLATFORM_TCHAR_IS_CHAR16						1
#define PLATFORM_HAS_BSD_SOCKET_FEATURE_IOCTL			1
#define PLATFORM_HAS_BSD_SOCKET_FEATURE_POLL			1
//#define PLATFORM_USE_PTHREADS							1
#define PLATFORM_MAX_FILEPATH_LENGTH_DEPRECATED			MAC_MAX_PATH
#define PLATFORM_SUPPORTS_TBB							1
#define PLATFORM_SUPPORTS_MESH_SHADERS					1
#define PLATFORM_SUPPORTS_BINDLESS_RENDERING			1
#define PLATFORM_SUPPORTS_GEOMETRY_SHADERS				1

#define PLATFORM_GLOBAL_LOG_CATEGORY					LogMac

#if WITH_EDITOR
#	define PLATFORM_FILE_READER_BUFFER_SIZE				(256*1024)
#endif

#ifdef PLATFORM_MAC_ARM64
#	define PLATFORM_CACHE_LINE_SIZE						128
#else
#	define PLATFORM_CACHE_LINE_SIZE						64
#endif

#define MAC_MAX_PATH									1024

#if UE_BUILD_DEBUG
#	define FORCEINLINE									inline 									/* Don't force code to be inline */
#else
#	define FORCEINLINE									inline __attribute__ ((always_inline))	/* Force code to be inline */
#endif