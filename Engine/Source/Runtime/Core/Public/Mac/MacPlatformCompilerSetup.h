// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	MacPlatformCompilerSetup.h: pragmas, version checks and other things for the Mac compiler
==============================================================================================*/

#pragma once

// In OS X 10.8 SDK gl3.h complains if gl.h was also included. Unfortunately, gl.h is included by Cocoa in CoreVideo.framework, so we need to disable this warning
#define GL_DO_NOT_WARN_IF_MULTI_GL_VERSION_HEADERS_INCLUDED

/**
 * We require at least Xcode 9.4 to compile
 */
static_assert((__clang_major__ > 9) || (__clang_major__ == 9 && __clang_minor__ >= 1), "Xcode 9.4 or newer is required to compile on Mac. Please make sure it's installed and selected as default using xcode-select tool.");
