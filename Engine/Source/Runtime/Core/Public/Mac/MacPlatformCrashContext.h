// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Apple/ApplePlatformCrashContext.h"

struct CORE_API FMacCrashContext : public FApplePlatformCrashContext
{
	/** Constructor */
	FMacCrashContext(ECrashContextType InType, const TCHAR* InErrorMessage);

	/** Copies the PLCrashReporter minidump */
	void CopyMinidump(char const* OutputPath, char const* InputPath) const;

	/** Generates the ensure/crash info into the given folder */
	void GenerateInfoInFolder(char const* const InfoFolder) const;
	
	/** Generates crash or ensure information for reporter */
	void GenerateInfoAndLaunchCrashReporter(bool IsCrash /* false means ensure */) const;
	
	/** Generates information for crash reporter */
	void GenerateCrashInfoAndLaunchReporter() const;
	
	/** Generates information for ensures sent via the CrashReporter */
	void GenerateEnsureInfoAndLaunchReporter() const;

	/** Captures all information about all threads */
	void CaptureAllThreadContext(uint32 ThreadIdEnteredOn) const;

protected:
	virtual bool GetPlatformAllThreadContextsString(FString& OutStr) const override;

private:
	void AddThreadContext(
		uint32 ThreadIdEnteredOn, 
		uint32 ThreadId,
		const FString& ThreadName,
		const TArray<FCrashStackFrame>& PortableCallStack) const;

	/**
	* <Thread>
	*   <CallStack>...</CallStack>
	*   <IsCrashed>...</IsCrashed>
	*   <Registers>...</Registers>
	*   <ThreadID>...</ThreadID>
	*   <ThreadName>...</ThreadName>
	* </Thead>
	* <Thread>...</Thread>
	* ...
	*/
};

typedef FMacCrashContext FPlatformCrashContext;
