// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	MacPlatformMath.h: Mac platform Math functions
==============================================================================================*/

#pragma once
#include "Clang/ClangPlatformMath.h"


#if PLATFORM_MAC_X86
	#include <smmintrin.h>
	#include "Math/UnrealPlatformMathSSE4.h"

	/*
	 * Mac implementation of the Math OS functions
	 */
	struct FMacPlatformMath : public TUnrealPlatformMathSSE4Base<FClangPlatformMath>
	{
		static UE_FORCEINLINE_HINT bool IsNaN(float A) { return isnan(A) != 0; }
		static UE_FORCEINLINE_HINT bool IsNaN(double A) { return isnan(A) != 0; }
		static UE_FORCEINLINE_HINT bool IsFinite(float A) { return isfinite(A); }
		static UE_FORCEINLINE_HINT bool IsFinite(double A) { return isfinite(A); }
	};

	typedef FMacPlatformMath FPlatformMath;
#else
	typedef FClangPlatformMath FPlatformMath;
#endif
