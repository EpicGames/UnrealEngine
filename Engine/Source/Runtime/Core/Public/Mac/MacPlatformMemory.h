// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	MacPlatformMemory.h: Mac platform memory functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformMemory.h"
#include "Apple/ApplePlatformMemory.h"

/**
* Mac implementation of the memory OS functions
**/
struct CORE_API FMacPlatformMemory : public FApplePlatformMemory
{
	/**
	 * Mac representation of a shared memory region
	 */
	struct FMacSharedMemoryRegion : public FSharedMemoryRegion
	{
		/** Returns file descriptor of a shared memory object */
		int GetFileDescriptor() const { return Fd; }

		/** Returns true if we need to unlink this region on destruction (no other process will be able to access it) */
		bool NeedsToUnlinkRegion() const { return bCreatedThisRegion; }

		FMacSharedMemoryRegion(const FString& InName, uint32 InAccessMode, void* InAddress, SIZE_T InSize, int InFd, bool bInCreatedThisRegion)
			:	FSharedMemoryRegion(InName, InAccessMode, InAddress, InSize)
			,	Fd(InFd)
			,	bCreatedThisRegion(bInCreatedThisRegion)
		{}

	protected:

		/** File descriptor of a shared region */
		int				Fd;

		/** Whether we created this region */
		bool			bCreatedThisRegion;
	};

	
	
	//~ Begin FGenericPlatformMemory Interface
	static FPlatformMemoryStats GetStats();
	static const FPlatformMemoryConstants& GetConstants();
	static FMalloc* BaseAllocator();
	
	static FGenericPlatformMemoryStats::EMemoryPressureStatus MemoryPressureStatus;
	
	static CORE_API FSharedMemoryRegion * MapNamedSharedMemoryRegion(const FString& InName, bool bCreate, uint32 AccessMode, SIZE_T Size);
	static CORE_API bool UnmapNamedSharedMemoryRegion(FSharedMemoryRegion * MemoryRegion);
	//~ End FGenericPlatformMemory Interface
};

typedef FMacPlatformMemory FPlatformMemory;



