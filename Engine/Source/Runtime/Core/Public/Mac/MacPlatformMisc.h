// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	MacPlatformMisc.h: Mac platform misc functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformMisc.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Mac/MacSystemIncludes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#ifdef __OBJC__
@class Protocol;
#endif

#include "Apple/ApplePlatformMisc.h"

class FMacGPUDescriptor;

/**
* Mac implementation of the misc OS functions
**/
struct CORE_API FMacPlatformMisc : public FApplePlatformMisc
{
	static void PlatformPreInit();
	static void PlatformInit();
	static void PlatformTearDown();

	UE_FORCEINLINE_HINT static constexpr int32 GetMaxPathLength()
	{
		return MAC_MAX_PATH;
	}

	static const TCHAR* GetPathVarDelimiter()
	{
		return TEXT(":");
	}

	static TArray<uint8> GetMacAddress();

	static void RequestExit(bool Force, const TCHAR* CallSite = nullptr);
	static EAppReturnType::Type MessageBoxExt( EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption );
	static bool CommandLineCommands();
	static int32 NumberOfCores();
	static int32 NumberOfCoresIncludingHyperthreads();
	static void NormalizePath(FString& InPath);
	static void NormalizePath(FStringBuilderBase& InPath);
	static FString GetPrimaryGPUBrand();
	static struct FGPUDriverInfo GetGPUDriverInfo(const FString& DeviceDescription, bool bVerbose = true);
	static void GetOSVersions( FString& out_OSVersionLabel, FString& out_OSSubVersionLabel );
	static FString GetOSVersion();
	static bool HasPlatformFeature(const TCHAR* FeatureName);
	static bool GetDiskTotalAndFreeSpace(const FString& InPath, uint64& TotalNumberOfBytes, uint64& NumberOfFreeBytes);
	static bool HasSeparateChannelForDebugOutput();
	static FString GetBonjourName();

	/** 
	 * Determines the shader format for the plarform
	 *
	 * @return	Returns the shader format to be used by that platform
	 */
	static CORE_API const TCHAR* GetNullRHIShaderFormat();

	/**
	 * Uses cpuid instruction to get the vendor string
	 *
	 * @return	CPU vendor name
	 */
	static FString GetCPUVendor();

	/**
	 * Uses cpuid instruction to get the CPU brand string
	 *
	 * @return    CPU brand string
	 */
	static FString GetCPUBrand();

	/**
	 * Uses cpuid instruction to get the vendor string
	 *
	 * @return	CPU info bitfield
	 *
	 *			Bits 0-3	Stepping ID
	 *			Bits 4-7	Model
	 *			Bits 8-11	Family
	 *			Bits 12-13	Processor type (Intel) / Reserved (AMD)
	 *			Bits 14-15	Reserved
	 *			Bits 16-19	Extended model
	 *			Bits 20-27	Extended family
	 *			Bits 28-31	Reserved
	 */
	static uint32 GetCPUInfo();

	static void SetGracefulTerminationHandler();
	
	static void SetCrashHandler(void (* CrashHandler)(const FGenericCrashContext& Context));

	/** @return Get the name of the platform specific file manager (Finder) */
	static FText GetFileManagerName();

	/**
	 * Returns whether the platform is running on battery power or not.
	 */
	static bool IsRunningOnBattery();

	static IPlatformChunkInstall* GetPlatformChunkInstall();

	/**
	 * Returns if current < target returns 1, if current > target returns 1, else current == target and returns 0.
	 */
	static int32 MacOSXVersionCompare(uint8 Major, uint8 Minor, uint8 Revision);

	/**
	 * Gets a globally unique ID the represents a particular operating system install.
	 */
	static FString GetOperatingSystemId();

	static FString GetXcodePath();

	/**
	 * Returns if current < target returns 1, if current > target returns 1, else current == target and returns 0.
	 */
	static int32 XCodeVersionCompare(uint8 Major, uint8 Minor, uint8 Revision);
	
	static bool IsSupportedXcodeVersionInstalled();

#if WITH_EDITOR
	static bool IsRunningOnRecommendedMinSpecHardware();
#endif // WITH_EDITOR

	static void MergeDefaultArgumentsIntoCommandLine(FString& CommandLine, FString DefaultArguments);
	
	enum class EMacGPUNotification : uint8
	{
		Added,
		RemovalRequested,
		Removed
	};
	
	/** Handle GPU change notifications. */
	static void GPUChangeNotification(uint64_t DeviceRegistryID, EMacGPUNotification Notification);
	
	using FGPUDescriptor = FMacGPUDescriptor;

	/** Returns the static list of GPUs in the current machine. */
	static TArray<FGPUDescriptor> const& GetGPUDescriptors();
	
	/** Returns the index of the GPU to use or -1 if we should just use the default. */
	static int32 GetExplicitRendererIndex();
    
    /** Update the driver monitor statistics for the given GPU - called once a frame by the Mac RHI's, no need to call otherwise - use GetPerformanceStatistics instead. */
    static void UpdateDriverMonitorStatistics(int32 DeviceIndex);

	static int GetDefaultStackSize();

	/** Updates variables in GMacAppInfo that cannot be initialized before PlatformPostInit() */
	static void PostInitMacAppInfoUpdate();

	inline static void ChooseHDRDeviceAndColorGamut(uint32 DeviceId, uint32 DisplayNitLevel, EDisplayOutputFormat& OutputDevice, EDisplayColorGamut& ColorGamut)
	{
		// ScRGB, 1000 or 2000 nits, DCI-P3
		OutputDevice = DisplayNitLevel == 1000 ? EDisplayOutputFormat::HDR_ACES_1000nit_ScRGB : EDisplayOutputFormat::HDR_ACES_2000nit_ScRGB;
		ColorGamut = EDisplayColorGamut::DCIP3_D65;
	}
};

typedef FMacPlatformMisc FPlatformMisc;

enum EMacModifierKeys
{
	MMK_RightCommand	= 0xF754,
	MMK_LeftCommand		= 0xF755,
	MMK_LeftShift		= 0xF756,
	MMK_CapsLock		= 0xF757,
	MMK_LeftAlt			= 0xF758,
	MMK_LeftControl		= 0xF759,
	MMK_RightShift		= 0xF760,
	MMK_RightAlt		= 0xF761,
	MMK_RightControl	= 0xF762
};
