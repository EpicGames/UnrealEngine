// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	MacPlatformMiscEx.h: Mac platform additional misc functions that requires sdk depdendencies
==============================================================================================*/

#pragma once

#include "Mac/MacSystemIncludes.h"

class FMacPlatformMiscEx
{
public:
	CORE_API static NSOperatingSystemVersion GetNSOperatingSystemVersion();
	CORE_API static CGDisplayModeRef GetSupportedDisplayMode(CGDirectDisplayID DisplayID, uint32 Width, uint32 Height);

	/**
	 * Returns if A < B returns -1, if A > B returns 1, else A == B and returns 0.
	 */
	CORE_API static int32 VersionCompare(const NSOperatingSystemVersion& VersionA, const NSOperatingSystemVersion& VersionB);

	CORE_API static id<NSObject> CommandletActivity;
};
