// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Apple/AppleRecursiveMutex.h"
#include "Containers/UnrealString.h"
#include "HAL/PThreadsSharedMutex.h"
#include "Misc/Timespan.h"

namespace UE
{

/** A system-wide mutex for Mac. Uses exclusive file locking. */
class FMacSystemWideMutex final
{
public:
	FMacSystemWideMutex(const FMacSystemWideMutex&) = delete;
	FMacSystemWideMutex& operator=(const FMacSystemWideMutex&) = delete;

	/** Construct a named, system-wide mutex and attempt to get access/ownership of it. */
	CORE_API explicit FMacSystemWideMutex(const FString& InName, FTimespan InTimeout = FTimespan::Zero());

	/** Destructor releases system-wide mutex if it is currently owned. */
	CORE_API ~FMacSystemWideMutex();

	/**
	 * Does the calling thread have ownership of the system-wide mutex?
	 *
	 * @return True if obtained. WARNING: Returns true for an owned but previously abandoned locks so shared resources can be in undetermined states. You must handle shared data robustly.
	 */
	CORE_API bool IsValid() const;

	/** Releases system-wide mutex if it is currently owned. */
	CORE_API void Release();

private:
	int32 FileHandle;
};

using FPlatformRecursiveMutex  = FAppleRecursiveMutex;
using FPlatformSharedMutex     = FPThreadsSharedMutex;
using FPlatformSystemWideMutex = FMacSystemWideMutex;

} // UE
