// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	MacPlatformOutputDevices.h: Mac platform OutputDevices functions
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformOutputDevices.h"

struct CORE_API FMacPlatformOutputDevices : public FGenericPlatformOutputDevices
{
	static FOutputDevice*			GetEventLog();
	static FOutputDeviceError*      GetError();
};

typedef FMacPlatformOutputDevices FPlatformOutputDevices;

