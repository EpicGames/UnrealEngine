// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	MacPlatformProcess.h: Mac platform Process functions
==============================================================================================*/

#pragma once

#include "Apple/ApplePlatformProcess.h"
#include "PosixOS/PosixOSPlatformProcess.h"
#include <sys/sysctl.h>

/** Wrapper around Unix pid_t. Should not be copyable as changes in the process state won't be properly propagated to all copies. */
struct FProcState
{
	FProcState(const FProcState&) = delete;
	FProcState& operator=(const FProcState&) = delete;

	/** Initialization constructor. */
	explicit FProcState(pid_t InProcessId);

	/** Destructor. */
	~FProcState();

	/** Getter for process id */
	UE_FORCEINLINE_HINT pid_t GetProcessId() const
	{
		return ProcessId;
	}

	/**
	 * Returns whether this process is running.
	 *
	 * @return true if running
	 */
	bool	IsRunning();

	/**
	 * Returns child's return code (only valid to call if not running)
	 *
	 * @param ReturnCode set to return code if not NULL (use the value only if method returned true)
	 *
	 * @return return whether we have the return code (we don't if it crashed)
	 */
	bool	GetReturnCode(int32* ReturnCodePtr);

	/**
	 * Waits for the process to end.
	 * Has a side effect (stores child's return code).
	 */
	void	Wait();

protected:  // the below is not a public API!

	/** Process id */
	pid_t	ProcessId;

	/** Whether the process has finished or not (cached) */
	bool	bIsRunning = true;

	/** Whether the process's return code has been collected */
	bool	bHasBeenWaitedFor = false;

	/** Return code of the process (if negative, means that process did not finish gracefully but was killed/crashed*/
	int32	ReturnCode = -1;
};

/** FProcHandle can be copied (and thus passed by value). */
struct FProcHandle
{
	/** Child proc state set from CreateProc() call */
	FProcState* 		ProcInfo;

	/** Pid of external process opened with OpenProcess() call.
	  * Added to FProcHandle so we don't have to special case FProcState with process
	  * we can only check for running state, and even then the PID could be reused so
	  * we don't ever want to terminate, etc.
	  */
	pid_t				OpenedPid;

	constexpr FProcHandle()
	:	ProcInfo(nullptr), OpenedPid(-1)
	{
	}

	FProcHandle(FProcState* InHandle)
	:	ProcInfo(InHandle), OpenedPid(-1)
	{
	}

	FProcHandle(pid_t InProcPid)
	:	ProcInfo(nullptr), OpenedPid(InProcPid)
	{
	}

	/** Accessors. */
	UE_FORCEINLINE_HINT pid_t Get() const
	{
		return ProcInfo ? ProcInfo->GetProcessId() : OpenedPid;
	}

	/** Resets the handle to invalid */
	inline void Reset()
	{
		ProcInfo = nullptr;
		OpenedPid = -1;
	}

	/** Checks the validity of handle */
	UE_FORCEINLINE_HINT bool IsValid() const
	{
		return ProcInfo != nullptr || OpenedPid != -1;
	}

	// the below is not part of FProcHandle API and is specific to Unix implementation
	UE_FORCEINLINE_HINT FProcState* GetProcessInfo() const
	{
		return ProcInfo;
	}
};

/**
 * Mac implementation of the Process OS functions
 **/
struct CORE_API FMacPlatformProcess : public FApplePlatformProcess
{
	struct FProcEnumInfo;

	/**
	 * Process enumerator.
	 */
	class CORE_API FProcEnumerator
	{
	public:
		// Constructor
		FProcEnumerator();
		FProcEnumerator(const FProcEnumerator&) = delete;
		FProcEnumerator& operator=(const FProcEnumerator&) = delete;

		// Destructor
		~FProcEnumerator();

		// Gets current process enumerator info.
		FProcEnumInfo GetCurrent() const;

		/**
		 * Moves current to the next process.
		 *
		 * @returns True if succeeded. False otherwise.
		 */
		bool MoveNext();

	private:
		// Array of process info structures.
		struct kinfo_proc* Processes;

		// Count of processes.
		uint32 ProcCount;

		// Current process index.
		uint32 CurrentProcIndex;
	};

	/**
	 * Process enumeration info structure.
	 */
	struct CORE_API FProcEnumInfo
	{
		friend FProcEnumInfo FMacPlatformProcess::FProcEnumerator::GetCurrent() const;

		// Gets process PID.
		uint32 GetPID() const;

		// Gets parent process PID.
		uint32 GetParentPID() const;

		// Gets process name. I.e. exec name.
		FString GetName() const;

		// Gets process full image path. I.e. full path of the exec file.
		FString GetFullPath() const;

	private:
		// Private constructor.
		FProcEnumInfo(struct kinfo_proc InProcInfo);

		// Process info struct.
		struct kinfo_proc ProcInfo;
	};

	static void* GetDllHandle( const TCHAR* Filename );
	static void FreeDllHandle( void* DllHandle );
	static void* GetDllExport( void* DllHandle, const TCHAR* ProcName );
	static uint32 GetCurrentProcessId();
	static uint32 GetCurrentCoreNumber();
	static const TCHAR* BaseDir();
	static const TCHAR* ComputerName();
	static const TCHAR* UserName(bool bOnlyAlphaNumeric = true);
	static void SetCurrentWorkingDirectoryToBaseDir();
	static FString GetCurrentWorkingDirectory();
	static const TCHAR* ExecutablePath();
	static const TCHAR* ExecutableName(bool bRemoveExtension = true);
	static FString GenerateApplicationPath( const FString& AppName, EBuildConfiguration BuildConfiguration);
	static const TCHAR* GetModuleExtension();
	static const TCHAR* GetBinariesSubdirectory();
	static const FString GetModulesDirectory();
	static bool CanLaunchURL(const TCHAR* URL);
	static void LaunchURL(const TCHAR* URL, const TCHAR* Parms, FString* Error);
	static FString GetGameBundleId();
	static FProcHandle CreateProc(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild = nullptr);
	static FProcHandle CreateProc(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild, void* PipeStdErrChild);
	static TTuple<FProcHandle, UE::HAL::EProcessId> CreateProc(const UE::HAL::FProcessStartInfo& StartInfo);
	static FProcHandle OpenProcess(uint32 ProcessID);
	static bool IsProcRunning( FProcHandle & ProcessHandle );
	static void WaitForProc( FProcHandle & ProcessHandle );
	static void CloseProc( FProcHandle & ProcessHandle );
	static void TerminateProc( FProcHandle & ProcessHandle, bool KillTree = false );
	static bool GetProcReturnCode( FProcHandle & ProcHandle, int32* ReturnCode );
	static bool IsApplicationRunning( const TCHAR* ProcName );
	static bool IsApplicationRunning( uint32 ProcessId );
	static FString GetApplicationName( uint32 ProcessId );
	static bool IsSandboxedApplication();
	static bool LaunchFileInDefaultExternalApplication( const TCHAR* FileName, const TCHAR* Parms = NULL, ELaunchVerb::Type Verb = ELaunchVerb::Open, bool bPromptToOpenOnFailure = true);
	static void ExploreFolder( const TCHAR* FilePath );
	static FRunnableThread* CreateRunnableThread();
	static void ClosePipe( void* ReadPipe, void* WritePipe );
	static bool CreatePipe(void*& ReadPipe, void*& WritePipe, bool bWritePipeLocal = false);
	static FString ReadPipe( void* ReadPipe );
	static bool ReadPipeToArray(void* ReadPipe, TArray<uint8> & Output);
	static bool WritePipe(void* WritePipe, const FString& Message, FString* OutWritten = nullptr);
	static bool WritePipe(void* WritePipe, const uint8* Data, const int32 DataLength, int32* OutDataLength = nullptr);
	static bool ExecProcess(const TCHAR* URL, const TCHAR* Params, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr, const TCHAR* OptionalWorkingDirectory = NULL, bool bShouldEndWithParentProcess = false);
	static void SetThreadAffinityMask(uint64 AffinityMask);
	
	/** Mixin some PosixOS functions */
	static inline bool IsFirstInstance()
	{
		return FPosixOSPlatformProcess::IsFirstInstance();
	}
	static void CeaseBeingFirstInstance()
	{
		FPosixOSPlatformProcess::CeaseBeingFirstInstance();
	}
	static TSharedPtr<IProcessSentinel> CreateProcessSentinelObject()
	{
		return FPosixOSPlatformProcess::CreateProcessSentinelObject();
	}
	

	// Mac specific
	static const TCHAR* UserPreferencesDir();
	static const TCHAR* UserLogsDir();

private:

	static FProcHandle CreateProcInternal(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeStdOutChild, void* PipeStdInChild, void* PipeStdErrChild, bool bInheritHandles);
};

typedef FMacPlatformProcess FPlatformProcess;
