// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	MacPlatformProperties.h - Basic static properties of a platform 
	These are shared between:
		the runtime platform - via FPlatformProperties
		the target platforms - via ITargetPlatform
==================================================================================*/

#pragma once

#include "GenericPlatform/GenericPlatformProperties.h"


/**
 * Implements Mac platform properties.
 */
template<bool HAS_EDITOR_DATA, bool IS_DEDICATED_SERVER, bool IS_CLIENT_ONLY>
struct FMacPlatformProperties
	: public FGenericPlatformProperties
{
	constexpr static UE_FORCEINLINE_HINT bool HasEditorOnlyData()
	{
		return HAS_EDITOR_DATA;
	}

	constexpr static UE_FORCEINLINE_HINT const char* IniPlatformName()
	{
		return "Mac";
	}

	constexpr static UE_FORCEINLINE_HINT const TCHAR* GetRuntimeSettingsClassName()
	{
		return TEXT("/Script/MacTargetPlatform.MacTargetSettings");
	}

	constexpr static UE_FORCEINLINE_HINT bool IsGameOnly()
	{
		return UE_GAME;
	}

	constexpr static UE_FORCEINLINE_HINT bool IsServerOnly()
	{
		return IS_DEDICATED_SERVER;
	}

	constexpr static UE_FORCEINLINE_HINT bool IsClientOnly()
	{
		return IS_CLIENT_ONLY;
	}

	constexpr static inline const char* PlatformName()
	{
		if (IS_DEDICATED_SERVER)
		{
			return "MacServer";
		}
		
		if (HAS_EDITOR_DATA)
		{
			return "MacEditor";
		}

		if (IS_CLIENT_ONLY)
		{
			return "MacClient";
		}

		return "Mac";
	}

	constexpr static UE_FORCEINLINE_HINT bool RequiresCookedData()
	{
		return !HAS_EDITOR_DATA;
	}

	constexpr static UE_FORCEINLINE_HINT bool HasSecurePackageFormat()
	{
		return IS_DEDICATED_SERVER;
	}

	constexpr static UE_FORCEINLINE_HINT bool SupportsMultipleGameInstances()
	{
		return false;
	}
	
	constexpr static UE_FORCEINLINE_HINT bool SupportsWindowedMode()
	{
		return true;
	}

	constexpr static UE_FORCEINLINE_HINT bool AllowsFramerateSmoothing()
	{
		return true;
	}

	constexpr static UE_FORCEINLINE_HINT bool HasFixedResolution()
	{
		return false;
	}

	constexpr static UE_FORCEINLINE_HINT bool SupportsQuit()
	{
		return true;
	}

	constexpr static inline float GetVariantPriority()
	{
		if (IS_DEDICATED_SERVER)
		{
			return 0.0f;
		}

		if (HAS_EDITOR_DATA)
		{
			return 0.0f;
		}

		if (IS_CLIENT_ONLY)
		{
			return 0.0f;
		}

		return 1.0f;
	}
	
	constexpr static UE_FORCEINLINE_HINT bool SupportsAudioStreaming()
	{
		return !IsServerOnly();
	}

	constexpr static UE_FORCEINLINE_HINT bool SupportsMeshLODStreaming()
	{
		return !IsServerOnly() && !HasEditorOnlyData();
	}
	
	static constexpr UE_FORCEINLINE_HINT bool SupportsRayTracing()
	{
		return true;
	}
};

#ifdef PROPERTY_HEADER_SHOULD_DEFINE_TYPE
typedef FMacPlatformProperties<WITH_EDITORONLY_DATA, UE_SERVER, !WITH_SERVER_CODE && !WITH_EDITOR> FPlatformProperties;
#endif
