// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Timespan.h"

class FMacSemaphore
{
public:
	UE_NONCOPYABLE(FMacSemaphore);

	CORE_API FMacSemaphore(int32 InitialCount, int32 /*MaxCount*/);
    CORE_API virtual ~FMacSemaphore();

	CORE_API void Acquire();
	CORE_API bool TryAcquire(FTimespan Timeout = FTimespan::Zero());
	CORE_API void Release(int32 Count = 1);

private:
	void* Semaphore; // (dispatch_semaphore_t)
};

typedef FMacSemaphore FSemaphore;
