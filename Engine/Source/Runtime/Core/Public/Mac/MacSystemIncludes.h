// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Set up compiler pragmas, etc
#include "CoreTypes.h"
#include "Mac/MacPlatformCompilerSetup.h"

#pragma push_macro("FVector")
#define FVector FVectorWorkaround
#ifdef __OBJC__
#import <Cocoa/Cocoa.h>
#undef PF_MAX
#endif
#include <Carbon/Carbon.h>
#include <CoreServices/CoreServices.h>

#pragma push_macro("check")
#pragma push_macro("verify")

#undef check
#undef verify
#undef FVector

#include <CoreFoundation/CoreFoundation.h>
#undef TYPE_BOOL

#include <string.h>
#include <alloca.h>
#include <stdlib.h>
#include <stdio.h>
#include <float.h>
#include <ctype.h>
#include <stdarg.h>
#include <stdint.h>
#include <limits.h>
#include <sys/time.h>
#include <math.h>
#include <mach/mach_time.h>
#include <wchar.h>
#include <wctype.h>
#include <errno.h>
#include <unistd.h>
#include <pthread.h>
#include <libkern/OSAtomic.h>
#include <sys/types.h>
#include <sys/stat.h>
#include <pwd.h>
#include <fcntl.h>
#include <dirent.h>
#include <dlfcn.h>
#include <copyfile.h>
#include <utime.h>
#include <mach/mach_host.h>
#include <mach/task.h>
#include <execinfo.h>
#include <signal.h>
#include <sys/sysctl.h>
#include <malloc/malloc.h>

#include "Apple/AppleTagRect.h"

/*----------------------------------------------------------------------------
Memory. On Mac OS X malloc allocates memory aligned to 16 bytes.
----------------------------------------------------------------------------*/
#define _aligned_malloc(Size,Align) malloc(Size)
#define _aligned_realloc(Ptr,Size,Align) realloc(Ptr,Size)
#define _aligned_free(Ptr) free(Ptr)
#define _aligned_msize(Ptr,Align,Offset) malloc_size(Ptr)

#pragma pop_macro("check")
#pragma pop_macro("verify")
#pragma pop_macro("FVector")
