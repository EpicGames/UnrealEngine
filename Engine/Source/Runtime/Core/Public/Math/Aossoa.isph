// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __AOSSOA_ISPH__
#define __AOSSOA_ISPH__

// Change layout from
// input: a0 b0 c0 d0 a1 b1 c1 d1 a2 b2 c2 d2 ...
// to
// output a0 a1 ... aX b0 b1 ... bX c0 c1 ... cX d0 d1 .. dX aX+1 ...

#if TARGET_WIDTH == 4
static const varying int vAOS20 = {0, 2, 4, 6};
static const varying int vAOS21 = {1, 3, 5, 7};
#elif TARGET_WIDTH == 8
static const varying int vAOS20 = {0, 2, 4, 6, 8, 10, 12, 14};
static const varying int vAOS21 = {1, 3, 5, 7, 9, 11, 13, 15};
#elif TARGET_WIDTH == 16
static const varying int vAOS20 = {0, 2, 4, 6, 8, 10, 12, 14, 16, 18, 20, 22, 24, 26, 28, 30};
static const varying int vAOS21 = {1, 3, 5, 7, 9, 11, 13, 15, 17, 19, 21, 23, 25, 27, 29, 31};
#else
#error "No implementation for this target"
#endif

template<typename T>
unmasked inline void aos_to_soa2_ispc(uniform T src[], varying T* uniform v0, varying T* uniform v1)
{
    const varying T src0 = src[programIndex];
    const varying T src1 = src[programCount + programIndex];
    *v0 = shuffle(src0, src1, vAOS20);
    *v1 = shuffle(src0, src1, vAOS21);
}

#if TARGET_WIDTH == 4
static const varying int vAOS30 = {0, 3, 1, 4};
static const varying int vAOS31 = {2, 5, 3, 6};
static const varying int vAOS32 = {0, 1, 4, 5};
static const varying int vAOS33 = {2, 3, 6, 7};
static const varying int vAOS34 = {2, 5, -1, -1};
static const varying int vAOS35 = {0, 1, 4, 7};
#elif TARGET_WIDTH == 8
static const varying int vAOS30 = {0, 3, 6, 9, 1, 4, 7, 10};
static const varying int vAOS31 = {4, 7, 10, 13, 5, 8, 11, 14};
static const varying int vAOS32 = {0, 1, 2, 3, 8, 9, 10, 11};
static const varying int vAOS33 = {4, 5, 6, 7, 12, 13, 14, 15};
static const varying int vAOS34 = {2, 5, 8, 11, 14, -1, -1, -1};
static const varying int vAOS35 = {0, 1, 2, 3, 4, 9, 12, 15};

static const varying int vAOS301 = {0, 1, 6, 7, 2, 3, 8, 9};
static const varying int vAOS311 = {4, 5, 10, 11, 6, 7, 12, 13};
static const varying int vAOS321 = {0, 1, 2, 3, 8, 9, 10, 11};
static const varying int vAOS331 = {4, 5, 6, 7, 12, 13, 14, 15};
static const varying int vAOS341 = {4, 5, 10, 11, -1, -1, -1, -1};
static const varying int vAOS351 = {0, 1, 2, 3, 8, 9, 14, 15};
#elif TARGET_WIDTH == 16
static const varying int vAOS30 = {0, 3, 6, 9, 12, 15, 18, 21, 1, 4, 7, 10, 13, 16, 19, 22};
static const varying int vAOS31 = {8, 11, 14, 17, 20, 23, 26, 29, 9, 12, 15, 18, 21, 24, 27, 30};
static const varying int vAOS32 = {0, 1, 2, 3, 4, 5, 6, 7, 16, 17, 18, 19, 20, 21, 22, 23};
static const varying int vAOS33 = {8, 9, 10, 11, 12, 13, 14, 15, 24, 25, 26, 27, 28, 29, 30, 31};
static const varying int vAOS34 = {2, 5, 8, 11, 14, 17, 20, 23, 26, 29, -1, -1, -1, -1, -1, -1};
static const varying int vAOS35 = {0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 16, 19, 22, 25, 28, 31};
#else
#error "No implementation for this target"
#endif

template<typename T>
unmasked inline void aos_to_soa3_ispc(uniform T src[], varying T* uniform v0, varying T* uniform v1,
    varying T* uniform v2)
{
    const varying T src0 = src[programIndex];
    const varying T src1 = src[programCount + programIndex];
    const varying T src2 = src[2 * programCount + programIndex];

    const varying T t0 = shuffle(src0, src1, vAOS30);
    const varying T t1 = shuffle(src1, src2, vAOS31);
    *v0 = shuffle(t0, t1, vAOS32);
    *v1 = shuffle(t0, t1, vAOS33);

    const varying T t2 = shuffle(src0, src1, vAOS34);
    *v2 = shuffle(t2, src2, vAOS35);
}

#if TARGET_WIDTH == 8
template<>
unmasked inline void aos_to_soa3_ispc<double>(uniform double src[], varying double* uniform v0, varying double* uniform v1,
    varying double* uniform v2)
{
    const uniform float* uniform srcflt = (const uniform float* uniform)src;
    const varying float src0 = srcflt[programIndex];
    const varying float src1 = srcflt[programCount + programIndex];
    const varying float src2 = srcflt[2 * programCount + programIndex];
    const varying float src3 = srcflt[3 * programCount + programIndex];
    const varying float src4 = srcflt[4 * programCount + programIndex];
    const varying float src5 = srcflt[5 * programCount + programIndex];

    const varying float t0 = shuffle(src0, src1, vAOS301);
    const varying float t1 = shuffle(src1, src2, vAOS311);
    const varying float s0 = shuffle(src3, src4, vAOS301);
    const varying float s1 = shuffle(src4, src5, vAOS311);
    const varying float t2 = shuffle(src0, src1, vAOS341);
    const varying float u2 = shuffle(t2, src2, vAOS351);
    const varying float s2 = shuffle(src3, src4, vAOS341);

    const varying float u0 = shuffle(t0, t1, vAOS321);
    const varying float w0 = shuffle(s0, s1, vAOS321);
    uniform double<TARGET_WIDTH> x0;
    *((varying float *uniform)&x0) = u0;
    *((varying float *uniform)&x0[TARGET_WIDTH/2]) = w0;
    *v0 = *((varying double* uniform)&x0);

    const varying float u1 = shuffle(t0, t1, vAOS331);
    const varying float w1 = shuffle(s0, s1, vAOS331);
    uniform double<TARGET_WIDTH> x1;
    *((varying float *uniform)&x1) = u1;
    *((varying float *uniform)&x1[TARGET_WIDTH/2]) = w1;
    *v1 = *((varying double* uniform)&x1);

    const varying float w2 = shuffle(s2, src5, vAOS351);
    uniform double<TARGET_WIDTH> x2;
    *((varying float *uniform)&x2) = u2;
    *((varying float *uniform)&x2[TARGET_WIDTH/2]) = w2;
    *v2 = *((varying double* uniform)&x2);
}

template<>
unmasked inline void aos_to_soa3_ispc<int64>(uniform int64 src[], varying int64* uniform v0, varying int64* uniform v1,
    varying int64* uniform v2)
{
    const uniform float* uniform srcflt = (const uniform float* uniform)src;
    const varying float src0 = srcflt[programIndex];
    const varying float src1 = srcflt[programCount + programIndex];
    const varying float src2 = srcflt[2 * programCount + programIndex];
    const varying float src3 = srcflt[3 * programCount + programIndex];
    const varying float src4 = srcflt[4 * programCount + programIndex];
    const varying float src5 = srcflt[5 * programCount + programIndex];

    const varying float t0 = shuffle(src0, src1, vAOS301);
    const varying float t1 = shuffle(src1, src2, vAOS311);
    const varying float s0 = shuffle(src3, src4, vAOS301);
    const varying float s1 = shuffle(src4, src5, vAOS311);
    const varying float t2 = shuffle(src0, src1, vAOS341);
    const varying float u2 = shuffle(t2, src2, vAOS351);
    const varying float s2 = shuffle(src3, src4, vAOS341);

    const varying float u0 = shuffle(t0, t1, vAOS321);
    const varying float w0 = shuffle(s0, s1, vAOS321);
    uniform int64<TARGET_WIDTH> x0;
    *((varying float* uniform) & x0) = u0;
    *((varying float* uniform) & x0[TARGET_WIDTH / 2]) = w0;
    *v0 = *((varying int64* uniform) & x0);

    const varying float u1 = shuffle(t0, t1, vAOS331);
    const varying float w1 = shuffle(s0, s1, vAOS331);
    uniform int64<TARGET_WIDTH> x1;
    *((varying float* uniform) & x1) = u1;
    *((varying float* uniform) & x1[TARGET_WIDTH / 2]) = w1;
    *v1 = *((varying int64* uniform) & x1);

    const varying float w2 = shuffle(s2, src5, vAOS351);
    uniform int64<TARGET_WIDTH> x2;
    *((varying float* uniform) & x2) = u2;
    *((varying float* uniform) & x2[TARGET_WIDTH / 2]) = w2;
    *v2 = *((varying int64* uniform) & x2);
}
#endif

#if TARGET_WIDTH == 4
static const varying int vAOS40 = {0, 4, 1, 5};
static const varying int vAOS41 = {2, 6, 3, 7};
static const varying int vAOS42 = {0, 1, 4, 5};
static const varying int vAOS43 = {2, 3, 6, 7};
#elif TARGET_WIDTH == 8
static const varying int vAOS40 = {0, 1, 2, 3, 8, 9, 10, 11};
static const varying int vAOS41 = {4, 5, 6, 7, 12, 13, 14, 15};
static const varying int vAOS42 = {0, 8, 1, 9, 4, 12, 5, 13};
static const varying int vAOS43 = {2, 10, 3, 11, 6, 14, 7, 15};
static const varying int vAOS44 = {0, 1, 8, 9, 4, 5, 12, 13};
static const varying int vAOS45 = {2, 3, 10, 11, 6, 7, 14, 15};

static const varying int vAOS401 = {0, 1, 8, 9, 4, 5, 12, 13};
static const varying int vAOS411 = {2, 3, 10, 11, 6, 7, 14, 15};
static const varying int vAOS421 = {0, 1, 2, 3, 8, 9, 10, 11};
static const varying int vAOS431 = {4, 5, 6, 7, 12, 13, 14, 15};
#elif TARGET_WIDTH == 16
static const varying int vAOS40 = {0, 16, 1, 17, 4, 20, 5, 21, 8, 24, 9, 25, 12, 28, 13, 29};
static const varying int vAOS41 = {2, 18, 3, 19, 6, 22, 7, 23, 10, 26, 11, 27, 14, 30, 15, 31};
static const varying int vAOS42 = {0, 4, 8, 12, 1, 5, 9, 13, 16, 20, 24, 28, 17, 21, 25, 29};
static const varying int vAOS43 = {2, 6, 10, 14, 3, 7, 11, 15, 18, 22, 26, 30, 19, 23, 27, 31};
#else
#error "No implementation for this target"
#endif

#if TARGET_WIDTH == 8
template<typename T>
unmasked inline void aos_to_soa4_ispc(uniform T src[], varying T* uniform v0, varying T* uniform v1,
    varying T* uniform v2, varying T* uniform v3)
{
    const varying T src0 = src[programIndex];
    const varying T src1 = src[programCount + programIndex];
    const varying T src2 = src[2 * programCount + programIndex];
    const varying T src3 = src[3 * programCount + programIndex];

    const varying T t0 = shuffle(src0, src2, vAOS40);
    const varying T t1 = shuffle(src0, src2, vAOS41);
    const varying T t2 = shuffle(src1, src3, vAOS40);
    const varying T t3 = shuffle(src1, src3, vAOS41);

    const varying T u0 = shuffle(t0, t1, vAOS42);
    const varying T u1 = shuffle(t0, t1, vAOS43);
    const varying T u2 = shuffle(t2, t3, vAOS42);
    const varying T u3 = shuffle(t2, t3, vAOS43);

    *v0 = shuffle(u0, u2, vAOS44);
    *v1 = shuffle(u0, u2, vAOS45);
    *v2 = shuffle(u1, u3, vAOS44);
    *v3 = shuffle(u1, u3, vAOS45);
}

template<>
unmasked inline void aos_to_soa4_ispc<double>(uniform double src[], varying double* uniform v0, varying double* uniform v1,
    varying double* uniform v2, varying double* uniform v3)
{
    const uniform float* uniform srcflt = (const uniform float* uniform)src;
    const varying float src0 = srcflt[programIndex];
    const varying float src1 = srcflt[programCount + programIndex];
    const varying float src2 = srcflt[2 * programCount + programIndex];
    const varying float src3 = srcflt[3 * programCount + programIndex];
    const varying float src4 = srcflt[4 * programCount + programIndex];
    const varying float src5 = srcflt[5 * programCount + programIndex];
    const varying float src6 = srcflt[6 * programCount + programIndex];
    const varying float src7 = srcflt[7 * programCount + programIndex];

    const varying float t0 = shuffle(src0, src1, vAOS401);
    const varying float t1 = shuffle(src0, src1, vAOS411);
    const varying float t2 = shuffle(src2, src3, vAOS401);
    const varying float t3 = shuffle(src2, src3, vAOS411);
    const varying float t4 = shuffle(src4, src5, vAOS401);
    const varying float t5 = shuffle(src4, src5, vAOS411);
    const varying float t6 = shuffle(src6, src7, vAOS401);
    const varying float t7 = shuffle(src6, src7, vAOS411);

    const varying float u0 = shuffle(t0, t2, vAOS421);
    const varying float w0 = shuffle(t4, t6, vAOS421);
    uniform double<TARGET_WIDTH> x0;
    *((varying float* uniform)&x0) = u0;
    *((varying float* uniform)&x0[TARGET_WIDTH/2]) = w0;
    *v0 = *((varying double* uniform)&x0);

    const varying float u1 = shuffle(t1, t3, vAOS421);
    const varying float w1 = shuffle(t5, t7, vAOS421);
    uniform double<TARGET_WIDTH> x1;
    *((varying float* uniform)&x1) = u1;
    *((varying float* uniform)&x1[TARGET_WIDTH/2]) = w1;
    *v1 = *((varying double* uniform)&x1);

    const varying float u2 = shuffle(t0, t2, vAOS431);
    const varying float w2 = shuffle(t4, t6, vAOS431);
    uniform double<TARGET_WIDTH> x2;
    *((varying float* uniform)&x2) = u2;
    *((varying float* uniform)&x2[TARGET_WIDTH/2]) = w2;
    *v2 = *((varying double* uniform)&x2);

    const varying float u3 = shuffle(t1, t3, vAOS431);
    const varying float w3 = shuffle(t5, t7, vAOS431);
    uniform double<TARGET_WIDTH> x3;
    *((varying float* uniform)&x3) = u3;
    *((varying float* uniform)&x3[TARGET_WIDTH/2]) = w3;
    *v3 = *((varying double* uniform)&x3);
}

template<>
unmasked inline void aos_to_soa4_ispc<int64>(uniform int64 src[], varying int64* uniform v0, varying int64* uniform v1,
    varying int64* uniform v2, varying int64* uniform v3)
{
    const uniform float* uniform srcflt = (const uniform float* uniform)src;
    const varying float src0 = srcflt[programIndex];
    const varying float src1 = srcflt[programCount + programIndex];
    const varying float src2 = srcflt[2 * programCount + programIndex];
    const varying float src3 = srcflt[3 * programCount + programIndex];
    const varying float src4 = srcflt[4 * programCount + programIndex];
    const varying float src5 = srcflt[5 * programCount + programIndex];
    const varying float src6 = srcflt[6 * programCount + programIndex];
    const varying float src7 = srcflt[7 * programCount + programIndex];

    const varying float t0 = shuffle(src0, src1, vAOS401);
    const varying float t1 = shuffle(src0, src1, vAOS411);
    const varying float t2 = shuffle(src2, src3, vAOS401);
    const varying float t3 = shuffle(src2, src3, vAOS411);
    const varying float t4 = shuffle(src4, src5, vAOS401);
    const varying float t5 = shuffle(src4, src5, vAOS411);
    const varying float t6 = shuffle(src6, src7, vAOS401);
    const varying float t7 = shuffle(src6, src7, vAOS411);

    const varying float u0 = shuffle(t0, t2, vAOS421);
    const varying float w0 = shuffle(t4, t6, vAOS421);
    uniform int64<TARGET_WIDTH> x0;
    *((varying float* uniform) & x0) = u0;
    *((varying float* uniform) & x0[TARGET_WIDTH / 2]) = w0;
    *v0 = *((varying int64* uniform)&x0);

    const varying float u1 = shuffle(t1, t3, vAOS421);
    const varying float w1 = shuffle(t5, t7, vAOS421);
    uniform int64<TARGET_WIDTH> x1;
    *((varying float* uniform) & x1) = u1;
    *((varying float* uniform) & x1[TARGET_WIDTH / 2]) = w1;
    *v1 = *((varying int64* uniform)&x1);

    const varying float u2 = shuffle(t0, t2, vAOS431);
    const varying float w2 = shuffle(t4, t6, vAOS431);
    uniform int64<TARGET_WIDTH> x2;
    *((varying float* uniform) & x2) = u2;
    *((varying float* uniform) & x2[TARGET_WIDTH / 2]) = w2;
    *v2 = *((varying int64* uniform)&x2);

    const varying float u3 = shuffle(t1, t3, vAOS431);
    const varying float w3 = shuffle(t5, t7, vAOS431);
    uniform int64<TARGET_WIDTH> x3;
    *((varying float* uniform) & x3) = u3;
    *((varying float* uniform) & x3[TARGET_WIDTH / 2]) = w3;
    *v3 = *((varying int64* uniform)&x3);
}
#else
template<typename T>
unmasked inline void aos_to_soa4_ispc(uniform T src[], varying T *uniform v0, varying T *uniform v1,
    varying T *uniform v2, varying T *uniform v3)
{
    const varying T src0 = src[programIndex];
    const varying T src1 = src[programCount + programIndex];
    const varying T src2 = src[2 * programCount + programIndex];
    const varying T src3 = src[3 * programCount + programIndex];

    const varying T t0 = shuffle(src0, src1, vAOS40);
    const varying T t1 = shuffle(src0, src1, vAOS41);
    const varying T t2 = shuffle(src2, src3, vAOS40);
    const varying T t3 = shuffle(src2, src3, vAOS41);

    *v0 = shuffle(t0, t2, vAOS42);
    *v1 = shuffle(t0, t2, vAOS43);
    *v2 = shuffle(t1, t3, vAOS42);
    *v3 = shuffle(t1, t3, vAOS43);
}
#endif

#if TARGET_WIDTH == 4
static const varying int vAOS60 = {0, 1, 4, 5};
static const varying int vAOS61 = {2, 3, 6, 7};
static const varying int vAOS62 = {0, 6, 1, 7};
static const varying int vAOS63 = {2, 4, 3, 5};
#elif TARGET_WIDTH == 8
static const varying int vAOS60 = {0, 1, 2, 3, 8, 9, 10, 11};
static const varying int vAOS61 = {4, 5, 6, 7, 12, 13, 14, 15};
static const varying int vAOS62 = {0, 6, 12, -1, 1, 7, 13, -1};
static const varying int vAOS63 = {0, 1, 2, 10, 4, 5, 6, 11};
static const varying int vAOS64 = {2, 8, 14, -1, 3, 9, 15, -1};
static const varying int vAOS65 = {0, 1, 2, 12, 4, 5, 6, 13};
static const varying int vAOS66 = {4, 10, -1, -1, 5, 11, -1, -1};
static const varying int vAOS67 = {0, 1, 8, 14, 4, 5, 9, 15};
#elif TARGET_WIDTH == 16
static const varying int vAOS60 = {0, 1, 2, 3, 4, 5, 6, 7, 16, 17, 18, 19, 20, 21, 22, 23};
static const varying int vAOS61 = {8, 9, 10, 11, 12, 13, 14, 15, 24, 25, 26, 27, 28, 29, 30, 31};
static const varying int vAOS62 = {0, 6, 12, 18, 24, 30, -1, -1, 1, 7, 13, 19, 25, 31, -1, -1};
static const varying int vAOS63 = {0, 1, 2, 3, 4, 5, 20, 26, 8, 9, 10, 11, 12, 13, 21, 27};
static const varying int vAOS64 = {2, 8, 14, 20, 26, -1, -1, 15, 3, 9, 15, 21, 27, -1, -1, 31};
static const varying int vAOS65 = {0, 1, 2, 3, 4, 16, 22, 28, 8, 9, 10, 11, 12, 17, 23, 29};
static const varying int vAOS66 = {4, 10, 16, 22, 28, -1, -1, -1, 5, 11, 17, 23, 29, -1, -1, -1};
static const varying int vAOS67 = {0, 1, 2, 3, 4, 18, 24, 30, 8, 9, 10, 11, 12, 19, 25, 31};
#else
#error "No implementation for this target"
#endif

#if TARGET_WIDTH == 4
template<typename T>
inline void aos_to_soa6_ispc(uniform T src[], varying T *uniform v0, varying T *uniform v1, varying T *uniform v2, varying T *uniform v3, varying T *uniform v4, varying T *uniform v5)                                  \
{
    const varying T src0 = src[programIndex];
    const varying T src1 = src[programCount + programIndex];
    const varying T src2 = src[2 * programCount + programIndex];
    const varying T src3 = src[3 * programCount + programIndex];
    const varying T src4 = src[4 * programCount + programIndex];
    const varying T src5 = src[5 * programCount + programIndex];
                                                     
    const varying T t0 = shuffle(src0, src1, vAOS62);
    const varying T t1 = shuffle(src3, src4, vAOS62);
    const varying T t2 = shuffle(src0, src2, vAOS63);
    const varying T t3 = shuffle(src3, src5, vAOS63);
    const varying T t4 = shuffle(src1, src2, vAOS62);
    const varying T t5 = shuffle(src4, src5, vAOS62);

    *v0 = shuffle(t0, t1, vAOS60);
    *v1 = shuffle(t0, t1, vAOS61);
    *v2 = shuffle(t2, t3, vAOS60);
    *v3 = shuffle(t2, t3, vAOS61);
    *v4 = shuffle(t4, t5, vAOS60);
    *v5 = shuffle(t4, t5, vAOS61);
}
#else
template<typename T>
unmasked inline void aos_to_soa6_ispc(uniform T src[], varying T *uniform v0, varying T *uniform v1, varying T *uniform v2, varying T *uniform v3, varying T *uniform v4, varying T *uniform v5)                                  \
{
    const varying T src0 = src[programIndex];
    const varying T src1 = src[programCount + programIndex];
    const varying T src2 = src[2 * programCount + programIndex];
    const varying T src3 = src[3 * programCount + programIndex];
    const varying T src4 = src[4 * programCount + programIndex];
    const varying T src5 = src[5 * programCount + programIndex];

    const varying T t0 = shuffle(shuffle(src0, src1, vAOS62), src2, vAOS63);
    const varying T t1 = shuffle(shuffle(src3, src4, vAOS62), src5, vAOS63);
    const varying T t2 = shuffle(shuffle(src0, src1, vAOS64), src2, vAOS65);
    const varying T t3 = shuffle(shuffle(src3, src4, vAOS64), src5, vAOS65);
    const varying T t4 = shuffle(shuffle(src0, src1, vAOS66), src2, vAOS67);
    const varying T t5 = shuffle(shuffle(src3, src4, vAOS66), src5, vAOS67);

    *v0 = shuffle(t0, t1, vAOS60);
    *v1 = shuffle(t0, t1, vAOS61);
    *v2 = shuffle(t2, t3, vAOS60);
    *v3 = shuffle(t2, t3, vAOS61);
    *v4 = shuffle(t4, t5, vAOS60);
    *v5 = shuffle(t4, t5, vAOS61);
}
#endif

template<typename T>
unmasked inline void UniformLoad(const uniform T* varying SrcPtr, uniform T* uniform DstPtr)
{
    uniform T* uniform Src[programCount];

    foreach(i = 0 ... programCount)
    {
        Src[i] = (uniform T* varying)&SrcPtr[0];
    }

    DstPtr[0] = *Src[0];
    DstPtr[1] = *Src[1];
    DstPtr[2] = *Src[2];
    DstPtr[3] = *Src[3];
#if TARGET_WIDTH == 8 || TARGET_WIDTH == 16
    DstPtr[4] = *Src[4];
    DstPtr[5] = *Src[5];
    DstPtr[6] = *Src[6];
    DstPtr[7] = *Src[7];
#endif
#if TARGET_WIDTH == 16
    DstPtr[8] = *Src[8];
    DstPtr[9] = *Src[9];
    DstPtr[10] = *Src[10];
    DstPtr[11] = *Src[11];
    DstPtr[12] = *Src[12];
    DstPtr[13] = *Src[13];
    DstPtr[14] = *Src[14];
    DstPtr[15] = *Src[15];
#endif
}

template<typename T>
unmasked inline void AosToSoa3Explicit(const uniform T* varying SrcPtr, uniform T* uniform DstPtr)
{
    uniform T* uniform Src[programCount];

    foreach(i = 0 ... programCount)
    {
        Src[i] = (uniform T * varying)&SrcPtr[0];
    }

#if TARGET_WIDTH == 4
    // X, X, X, X
    DstPtr[0] = Src[0][0];
    DstPtr[1] = Src[1][0];
    DstPtr[2] = Src[2][0];
    DstPtr[3] = Src[3][0];

    // Y, Y, Y, Y
    DstPtr[4] = Src[0][1];
    DstPtr[5] = Src[1][1];
    DstPtr[6] = Src[2][1];
    DstPtr[7] = Src[3][1];

    // Z, Z, Z, Z
    DstPtr[8] = Src[0][2];
    DstPtr[9] = Src[1][2];
    DstPtr[10] = Src[2][2];
    DstPtr[11] = Src[3][2];
#elif TARGET_WIDTH == 8
    // X, X, X, X, X, X, X, X
    DstPtr[0] = Src[0][0];
    DstPtr[1] = Src[1][0];
    DstPtr[2] = Src[2][0];
    DstPtr[3] = Src[3][0];
    DstPtr[4] = Src[4][0];
    DstPtr[5] = Src[5][0];
    DstPtr[6] = Src[6][0];
    DstPtr[7] = Src[7][0];

    // Y, Y, Y, Y, Y, Y, Y, Y
    DstPtr[8] = Src[0][1];
    DstPtr[9] = Src[1][1];
    DstPtr[10] = Src[2][1];
    DstPtr[11] = Src[3][1];
    DstPtr[12] = Src[4][1];
    DstPtr[13] = Src[5][1];
    DstPtr[14] = Src[6][1];
    DstPtr[15] = Src[7][1];

    // Z, Z, Z, Z, Z, Z, Z, Z
    DstPtr[16] = Src[0][2];
    DstPtr[17] = Src[1][2];
    DstPtr[18] = Src[2][2];
    DstPtr[19] = Src[3][2];
    DstPtr[20] = Src[4][2];
    DstPtr[21] = Src[5][2];
    DstPtr[22] = Src[6][2];
    DstPtr[23] = Src[7][2];
#elif TARGET_WIDTH == 16
    // X, X, X, X, X, X, X, X, X, X, X, X, X, X, X, X
    DstPtr[0] = Src[0][0];
    DstPtr[1] = Src[1][0];
    DstPtr[2] = Src[2][0];
    DstPtr[3] = Src[3][0];
    DstPtr[4] = Src[4][0];
    DstPtr[5] = Src[5][0];
    DstPtr[6] = Src[6][0];
    DstPtr[7] = Src[7][0];
    DstPtr[8] = Src[8][0];
    DstPtr[9] = Src[9][0];
    DstPtr[10] = Src[10][0];
    DstPtr[11] = Src[11][0];
    DstPtr[12] = Src[12][0];
    DstPtr[13] = Src[13][0];
    DstPtr[14] = Src[14][0];
    DstPtr[15] = Src[15][0];

    // Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y
    DstPtr[16] = Src[0][1];
    DstPtr[17] = Src[1][1];
    DstPtr[18] = Src[2][1];
    DstPtr[19] = Src[3][1];
    DstPtr[20] = Src[4][1];
    DstPtr[21] = Src[5][1];
    DstPtr[22] = Src[6][1];
    DstPtr[23] = Src[7][1];
    DstPtr[24] = Src[8][1];
    DstPtr[25] = Src[9][1];
    DstPtr[26] = Src[10][1];
    DstPtr[27] = Src[11][1];
    DstPtr[28] = Src[12][1];
    DstPtr[29] = Src[13][1];
    DstPtr[30] = Src[14][1];
    DstPtr[31] = Src[15][1];

    // Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z
    DstPtr[32] = Src[0][2];
    DstPtr[33] = Src[1][2];
    DstPtr[34] = Src[2][2];
    DstPtr[35] = Src[3][2];
    DstPtr[36] = Src[4][2];
    DstPtr[37] = Src[5][2];
    DstPtr[38] = Src[6][2];
    DstPtr[39] = Src[7][2];
    DstPtr[40] = Src[8][2];
    DstPtr[41] = Src[9][2];
    DstPtr[42] = Src[10][2];
    DstPtr[43] = Src[11][2];
    DstPtr[44] = Src[12][2];
    DstPtr[45] = Src[13][2];
    DstPtr[46] = Src[14][2];
    DstPtr[47] = Src[15][2];
#endif
}

template<typename T>
unmasked inline void AosToSoa4Explicit(const uniform T* varying SrcPtr, uniform T* uniform DstPtr)
{
    uniform T* uniform Src[programCount];

    foreach(i = 0 ... programCount)
    {
        Src[i] = (uniform T* varying)&SrcPtr[0];
    }

#if TARGET_WIDTH == 4
    // X, X, X, X
    DstPtr[0] = Src[0][0];
    DstPtr[1] = Src[1][0];
    DstPtr[2] = Src[2][0];
    DstPtr[3] = Src[3][0];

    // Y, Y, Y, Y
    DstPtr[4] = Src[0][1];
    DstPtr[5] = Src[1][1];
    DstPtr[6] = Src[2][1];
    DstPtr[7] = Src[3][1];

    // Z, Z, Z, Z
    DstPtr[8] = Src[0][2];
    DstPtr[9] = Src[1][2];
    DstPtr[10] = Src[2][2];
    DstPtr[11] = Src[3][2];

    // W, W, W, W
    DstPtr[12] = Src[0][3];
    DstPtr[13] = Src[1][3];
    DstPtr[14] = Src[2][3];
    DstPtr[15] = Src[3][3];
#elif TARGET_WIDTH == 8
    // X, X, X, X, X, X, X, X
    DstPtr[0] = Src[0][0];
    DstPtr[1] = Src[1][0];
    DstPtr[2] = Src[2][0];
    DstPtr[3] = Src[3][0];
    DstPtr[4] = Src[4][0];
    DstPtr[5] = Src[5][0];
    DstPtr[6] = Src[6][0];
    DstPtr[7] = Src[7][0];

    // Y, Y, Y, Y, Y, Y, Y, Y
    DstPtr[8] = Src[0][1];
    DstPtr[9] = Src[1][1];
    DstPtr[10] = Src[2][1];
    DstPtr[11] = Src[3][1];
    DstPtr[12] = Src[4][1];
    DstPtr[13] = Src[5][1];
    DstPtr[14] = Src[6][1];
    DstPtr[15] = Src[7][1];

    // Z, Z, Z, Z, Z, Z, Z, Z
    DstPtr[16] = Src[0][2];
    DstPtr[17] = Src[1][2];
    DstPtr[18] = Src[2][2];
    DstPtr[19] = Src[3][2];
    DstPtr[20] = Src[4][2];
    DstPtr[21] = Src[5][2];
    DstPtr[22] = Src[6][2];
    DstPtr[23] = Src[7][2];

    // W, W, W, W, W, W, W, W
    DstPtr[24] = Src[0][3];
    DstPtr[25] = Src[1][3];
    DstPtr[26] = Src[2][3];
    DstPtr[27] = Src[3][3];
    DstPtr[28] = Src[4][3];
    DstPtr[29] = Src[5][3];
    DstPtr[30] = Src[6][3];
    DstPtr[31] = Src[7][3];
#elif TARGET_WIDTH == 16
    // X, X, X, X, X, X, X, X, X, X, X, X, X, X, X, X
    DstPtr[0] = Src[0][0];
    DstPtr[1] = Src[1][0];
    DstPtr[2] = Src[2][0];
    DstPtr[3] = Src[3][0];
    DstPtr[4] = Src[4][0];
    DstPtr[5] = Src[5][0];
    DstPtr[6] = Src[6][0];
    DstPtr[7] = Src[7][0];
    DstPtr[8] = Src[8][0];
    DstPtr[9] = Src[9][0];
    DstPtr[10] = Src[10][0];
    DstPtr[11] = Src[11][0];
    DstPtr[12] = Src[12][0];
    DstPtr[13] = Src[13][0];
    DstPtr[14] = Src[14][0];
    DstPtr[15] = Src[15][0];

    // Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y
    DstPtr[16] = Src[0][1];
    DstPtr[17] = Src[1][1];
    DstPtr[18] = Src[2][1];
    DstPtr[19] = Src[3][1];
    DstPtr[20] = Src[4][1];
    DstPtr[21] = Src[5][1];
    DstPtr[22] = Src[6][1];
    DstPtr[23] = Src[7][1];
    DstPtr[24] = Src[8][1];
    DstPtr[25] = Src[9][1];
    DstPtr[26] = Src[10][1];
    DstPtr[27] = Src[11][1];
    DstPtr[28] = Src[12][1];
    DstPtr[29] = Src[13][1];
    DstPtr[30] = Src[14][1];
    DstPtr[31] = Src[15][1];

    // Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z
    DstPtr[32] = Src[0][2];
    DstPtr[33] = Src[1][2];
    DstPtr[34] = Src[2][2];
    DstPtr[35] = Src[3][2];
    DstPtr[36] = Src[4][2];
    DstPtr[37] = Src[5][2];
    DstPtr[38] = Src[6][2];
    DstPtr[39] = Src[7][2];
    DstPtr[40] = Src[8][2];
    DstPtr[41] = Src[9][2];
    DstPtr[42] = Src[10][2];
    DstPtr[43] = Src[11][2];
    DstPtr[44] = Src[12][2];
    DstPtr[45] = Src[13][2];
    DstPtr[46] = Src[14][2];
    DstPtr[47] = Src[15][2];

    // W, W, W, W, W, W, W, W, W, W, W, W, W, W, W, W
    DstPtr[48] = Src[0][3];
    DstPtr[49] = Src[1][3];
    DstPtr[50] = Src[2][3];
    DstPtr[51] = Src[3][3];
    DstPtr[52] = Src[4][3];
    DstPtr[53] = Src[5][3];
    DstPtr[54] = Src[6][3];
    DstPtr[55] = Src[7][3];
    DstPtr[56] = Src[8][3];
    DstPtr[57] = Src[9][3];
    DstPtr[58] = Src[10][3];
    DstPtr[59] = Src[11][3];
    DstPtr[60] = Src[12][3];
    DstPtr[61] = Src[13][3];
    DstPtr[62] = Src[14][3];
    DstPtr[64] = Src[15][3];
#endif
}

#endif
