// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "UObject/CoreReflectedTypeAccessors.h"

// Generic axis enum (mirrored for property use in Object.h)
namespace EAxis
{
	enum Type : int
	{
		None,
		X,
		Y,
		Z,
	};
}

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<EAxis::Type>();
#endif


// Extended axis enum for more specialized usage
namespace EAxisList
{
	enum Type : int
	{
		None		= 0,
		X			= 1 << 0,
		Y			= 1 << 1,
		Z			= 1 << 2,

		Screen		= 1 << 3,
		XY			= X | Y,
		XZ			= X | Z,
		YZ			= Y | Z,
		XYZ			= X | Y | Z,
		All			= XYZ | Screen,

		//alias over Axis YZ since it isn't used when the z-rotation widget is being used
		ZRotation	= YZ,
		
		// alias over Screen since it isn't used when the 2d translate rotate widget is being used
		Rotate2D	= Screen,

		Left			= 1 << 4,
		Up				= 1 << 5,
		Forward			= 1 << 6,

		LU = Left | Up,
		LF = Left | Forward,
		UF = Up | Forward,
		LeftUpForward	= Left | Up | Forward,
	};
}

#ifdef COREUOBJECT_API
template<> COREUOBJECT_NON_ATTRIBUTED_API UEnum* StaticEnum<EAxisList::Type>();
#endif

namespace EAxis
{
	// Converts the given AxisList value to the corresponding Axis value. Assumes a single axis within the provided AxisList.
	inline EAxis::Type FromAxisList(const EAxisList::Type InAxisList)
	{
		PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
		switch (InAxisList)
		{
		case EAxisList::X:
		case EAxisList::Forward:
			return EAxis::X;

		case EAxisList::Y:
		case EAxisList::Left:
			return EAxis::Y;

		case EAxisList::Z:
		case EAxisList::Up:
			return EAxis::Z;

		case EAxisList::None:
		default:
			return EAxis::None;
		}
		PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	}
}

namespace EAxisList
{
	// Converts the given Axis to the corresponding AxisList value
	inline EAxisList::Type FromAxis(const EAxis::Type InAxis, const EAxisList::Type InAxisCoordinateSystem = EAxisList::XYZ)
	{
		if (InAxisCoordinateSystem == EAxisList::XYZ)
		{
			switch (InAxis)
			{
			case EAxis::X:
				return EAxisList::X;
			case EAxis::Y:
				return EAxisList::Y;
			case EAxis::Z:
				return EAxisList::Z;
			case EAxis::None:
			default:
				return EAxisList::None;
			}
		}

		if (InAxisCoordinateSystem == EAxisList::LeftUpForward)
		{
			switch (InAxis)
			{
			case EAxis::X:
				return EAxisList::Forward;
			case EAxis::Y:
				return EAxisList::Left;
			case EAxis::Z:
				return EAxisList::Up;
			case EAxis::None:
			default:
				return EAxisList::None;
			}
		}

		return EAxisList::None;
	}
}
