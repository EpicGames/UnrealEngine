// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Misc/ExpressionParserTypes.h"
#include "Misc/Optional.h"
#include "Templates/ValueOrError.h"

struct FDecimalNumberFormattingRules;

#define DEFINE_EXPRESSION_OPERATOR_NODE(EXPORTAPI, TYPE, ...) \
namespace ExpressionParser {\
	struct EXPORTAPI TYPE { static const TCHAR* const Moniker; }; \
}\
DEFINE_EXPRESSION_NODE_TYPE(ExpressionParser::TYPE, __VA_ARGS__)

/** Define some expression types for basic arithmetic */
DEFINE_EXPRESSION_OPERATOR_NODE(CORE_API, FSubExpressionStart, 0xCC40A083, 0xADBF46E2, 0xA93D12BB, 0x525D7417)
DEFINE_EXPRESSION_OPERATOR_NODE(CORE_API, FSubExpressionEnd, 0x125E4C67, 0x96EB48C4, 0x8894E09C, 0xB3CD56BF)

DEFINE_EXPRESSION_OPERATOR_NODE(CORE_API, FPlus, 0x6F88756B, 0xF9234263, 0x9B13614F, 0x2706074B)
DEFINE_EXPRESSION_OPERATOR_NODE(CORE_API, FPlusEquals, 0x05A878C9, 0x0E6C44A4, 0x9A73920D, 0x3175AB48)
DEFINE_EXPRESSION_OPERATOR_NODE(CORE_API, FMinus, 0xE6240779, 0xEE2849CF, 0x95A0E648, 0x22D58713)
DEFINE_EXPRESSION_OPERATOR_NODE(CORE_API, FMinusEquals, 0x8D1E08E3, 0x5F534245, 0xA987AD7E, 0xDB78A4B7)
DEFINE_EXPRESSION_OPERATOR_NODE(CORE_API, FStar, 0xF287B3BF, 0x35DF4141, 0xA8B4F57B, 0x7E06DF47)
DEFINE_EXPRESSION_OPERATOR_NODE(CORE_API, FStarEquals, 0xBA359BB0, 0xCEB54682, 0x9160EB4F, 0xD1687C7F)
DEFINE_EXPRESSION_OPERATOR_NODE(CORE_API, FForwardSlash, 0xF99670F8, 0x74794256, 0xBB0CAE6D, 0xC67CD5B6)
DEFINE_EXPRESSION_OPERATOR_NODE(CORE_API, FForwardSlashEquals, 0x4AFE0CF8, 0xF9054360, 0xBE5DCE80, 0xDC2E22F6)
DEFINE_EXPRESSION_OPERATOR_NODE(CORE_API, FPercent, 0x936E4434, 0x0A014F2D, 0xBEEC90D3, 0x4D3ECEA2)
DEFINE_EXPRESSION_OPERATOR_NODE(CORE_API, FSquareRoot, 0xE7C03E11, 0x9DE84B4B, 0xBA4C2B76, 0x69BF028E)
DEFINE_EXPRESSION_OPERATOR_NODE(CORE_API, FPower, 0x93388F8D, 0x1D9B4DFE, 0xBD4D6CC4, 0x12D1DE99)
	
namespace ExpressionParser
{
	/** Get the default set number formatting rules based on the current locale and user settings */
	CORE_API const FDecimalNumberFormattingRules& GetLocalizedNumberFormattingRules();

	/** Parse a number formatted using the given rules from the given stream, optionally from a specific read position */
	CORE_API TOptional<FStringToken> ParseNumberWithFallback(const FTokenStream& InStream, const FDecimalNumberFormattingRules& InPrimaryFormattingRules, const FDecimalNumberFormattingRules& InFallbackFormattingRules, FStringToken* Accumulate = nullptr, double* OutValue = nullptr);

	/** Parse a number formatted using the given rules from the given stream, optionally from a specific read position */
	CORE_API TOptional<FStringToken> ParseNumberWithRules(const FTokenStream& InStream, const FDecimalNumberFormattingRules& InFormattingRules, FStringToken* Accumulate = nullptr, double* OutValue = nullptr);

	/** Parse a localized number from the given stream, optionally from a specific read position */
	CORE_API TOptional<FStringToken> ParseLocalizedNumberWithAgnosticFallback(const FTokenStream& InStream, FStringToken* Accumulate = nullptr, double* OutValue = nullptr);

	/** Parse a localized number from the given stream, optionally from a specific read position */
	CORE_API TOptional<FStringToken> ParseLocalizedNumber(const FTokenStream& InStream, FStringToken* Accumulate = nullptr, double* OutValue = nullptr);

	/** Parse a number from the given stream, optionally from a specific read position */
	CORE_API TOptional<FStringToken> ParseNumber(const FTokenStream& InStream, FStringToken* Accumulate = nullptr, double* OutValue = nullptr);

	/** Consume a number formatted using the given rules from the specified consumer's stream, if one exists at the current read position */
	CORE_API TOptional<FExpressionError> ConsumeNumberWithRules(FExpressionTokenConsumer& Consumer, const FDecimalNumberFormattingRules& InFormattingRules);

	/** Consume a localized number from the specified consumer's stream, if one exists at the current read position */
	CORE_API TOptional<FExpressionError> ConsumeLocalizedNumberWithAgnosticFallback(FExpressionTokenConsumer& Consumer);

	/** Consume a localized number from the specified consumer's stream, if one exists at the current read position */
	CORE_API TOptional<FExpressionError> ConsumeLocalizedNumber(FExpressionTokenConsumer& Consumer);

	/** Consume a number from the specified consumer's stream, if one exists at the current read position */
	CORE_API TOptional<FExpressionError> ConsumeNumber(FExpressionTokenConsumer& Consumer);

	/** Consume a symbol from the specified consumer's stream, if one exists at the current read position */
	template<typename TSymbol>
	TOptional<FExpressionError> ConsumeSymbol(FExpressionTokenConsumer& Consumer)
	{
		TOptional<FStringToken> Token = Consumer.GetStream().ParseToken(TSymbol::Moniker);
		if (Token.IsSet())
		{
			Consumer.Add(Token.GetValue(), TSymbol());
		}

		return TOptional<FExpressionError>();
	}
}

/** A basic math expression evaluator */
class FBasicMathExpressionEvaluator
{
public:
	/** Constructor that sets up the parser's lexer and compiler */
	[[nodiscard]] CORE_API FBasicMathExpressionEvaluator();

	/** Evaluate the given expression, resulting in either a double value, or an error */
	[[nodiscard]] CORE_API TValueOrError<double, FExpressionError> Evaluate(const TCHAR* InExpression, double InExistingValue = 0) const;

	/**
	 * Utils that can be re-used as part of more complex evaluators
	 * 
	 * Note precedences are 3-5 for power/multiplicative/additive 
	 */
	CORE_API static void DefineArithmeticTokens(FTokenDefinitions& TokenDefinitions);
	CORE_API static void DefineArithmeticOperators(FExpressionGrammar& TokenDefinitions);
	CORE_API static void MapArithmeticOperators(FOperatorJumpTable& JumpTable);

private:
	FTokenDefinitions TokenDefinitions;
	FExpressionGrammar Grammar;
	FOperatorJumpTable JumpTable;
};
