// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Containers/UnrealString.h"

/**
 * n-bit integer. @todo: optimize
 * Data is stored as an array of 32-bit words from the least to the most significant
 * Doesn't handle overflows (not a big issue, we can always use a bigger bit count)
 * Minimum sanity checks.
 * Can convert from int64 and back (by truncating the result, this is mostly for testing)
 */
template <int32 NumBits, bool bSigned = true>
class TBigInt
{
public:

	typedef TBigInt<NumBits, bSigned> BigInt;

	static BigInt One;

private:

	enum
	{
		/** Word size. */
		BitsPerWord = 32,
		NumWords    = NumBits / BitsPerWord
	};

	static_assert(NumBits >= 64, "TBigInt must have at least 64 bits.");

	/** All bits stored as an array of words. */
	uint32 Bits[NumWords];
	
	/**
	 * Makes sure both factors are positive integers and stores their original signs
	 *
	 * @param FactorA first factor
	 * @param SignA sign of the first factor
	 * @param FactorB second factor
	 * @param SignB sign of the second pactor
	 */
	inline static void MakePositiveFactors(BigInt& FactorA, int32& SignA, BigInt& FactorB, int32& SignB)
	{
		if (bSigned)
	  {
		  SignA = FactorA.Sign();
		  SignB = FactorB.Sign();
		  if (SignA < 0)
		  {
			  FactorA.Negate();
		  }
		  if (SignB < 0)
		  {
			  FactorB.Negate();
		  }
	  }
	}

	/**
	 * Restores a sign of a result based on the sign of two factors that produced the result.
	 *
	 * @param Result math opration result
	 * @param SignA sign of the first factor
	 * @param SignB sign of the second factor
	 */
	inline static void RestoreSign(BigInt& Result, int32 SignA, int32 SignB)
	{
		if (bSigned && (SignA * SignB) < 0)
		{
			Result.Negate();
		}
	}

public:

	[[nodiscard]] UE_FORCEINLINE_HINT uint32* GetBits()
	{
		return Bits;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT const uint32* GetBits() const
	{
		return Bits;
	}

	/** Sets this integer to 0. */
	UE_FORCEINLINE_HINT void Zero()
	{
		FMemory::Memset(Bits, 0, sizeof(Bits));
	}

	/**
	 * Initializes this big int with a 64 bit integer value.
	 *
	 * @param Value The value to set.
	 */
	inline void Set(int64 Value)
	{
		Zero();
		Bits[0] = (Value & 0xffffffff);
		Bits[1] = (Value >> BitsPerWord) & 0xffffffff;
	}

	/** Default constructor. Initializes the number to zero. */
	[[nodiscard]] TBigInt()
	{
		Zero();
	}

	/**
	 * Constructor. Initializes this big int with a 64 bit integer value.
	 *
	 * @param Other The value to set.
	 */
	[[nodiscard]] TBigInt(int64 Other)
	{
		Set(Other);
	}

	/**
	 * Constructor. Initializes this big int with an array of words.
	 */
	[[nodiscard]] explicit TBigInt(const uint32* InBits)
	{
		FMemory::Memcpy(Bits, InBits, sizeof(Bits));
	}

	/**
	 * Constructor. Initializes this big int with an array of bytes.
	 */
	[[nodiscard]] explicit TBigInt(const uint8* InData, uint32 InNumBytes)
	{
		// If we weren't given enough data to fill the int, zero first
		// TODO: Only zero the bits we aren't going to copy over
		if (InNumBytes < (UE_ARRAY_COUNT(Bits) * sizeof(uint32)))
		{
			Zero();
		}

		FMemory::Memcpy(Bits, InData, InNumBytes);
	}

	/**
	 * Constructor. Initializes this big int with a string representing a hex value.
	 */
	[[nodiscard]] explicit TBigInt(const FString& Value)
	{
		Parse(Value);
	}

	/**
	 * Shift left by the specified amount of bits. Does not check if BitCount is valid.
	 *
	 * @param BitCount the number of bits to shift.
	 */
	void ShiftLeftInternal(const int32 BitCount)
	{
		checkSlow(BitCount > 0);

		BigInt Result;
		if (BitCount & (BitsPerWord - 1))
		{
			const int32 LoWordOffset = (BitCount - 1) / BitsPerWord;
			const int32 HiWordOffset = LoWordOffset + 1;
			const int32 LoWordShift  = BitCount - LoWordOffset * BitsPerWord;
			const int32 HiWordShift  = BitsPerWord - LoWordShift;
			Result.Bits[NumWords - 1] |= Bits[NumWords - HiWordOffset] << LoWordShift;
			for (int32 WordIndex = (NumWords - 1) - HiWordOffset; WordIndex >= 0; --WordIndex)
			{
				uint32 Value = Bits[WordIndex];
				Result.Bits[WordIndex + LoWordOffset] |= Value << LoWordShift;
				Result.Bits[WordIndex + HiWordOffset] |= Value >> HiWordShift;
			}
		}
		else
		{
			const int32 ShiftWords = BitCount / BitsPerWord;
			for (int32 WordIndex = NumWords - 1; WordIndex >= ShiftWords; --WordIndex)
			{
				Result.Bits[WordIndex] = Bits[WordIndex - ShiftWords];
			}
		}
		*this = Result;
	}

	/**
	 * Shift left by 1 bit.
	 */
	void ShiftLeftByOneInternal()
	{
		const int32 HiWordShift = BitsPerWord - 1;
		Bits[NumWords - 1] = Bits[NumWords - 1] << 1;
		for (int32 WordIndex = NumWords - 2; WordIndex >= 0; --WordIndex)
		{
			const uint32 Value = Bits[WordIndex];
			Bits[WordIndex + 0] = Value << 1;
			Bits[WordIndex + 1] |= Value >> HiWordShift;
		}
	}

	/**
	 * Shift right by the specified amount of bits. Does not check if BitCount is valid.
	 *
	 * @param BitCount the number of bits to shift.
	 */
	void ShiftRightInternal(const int32 BitCount)
	{
		checkSlow(BitCount > 0);

		BigInt Result;
		if (BitCount & (BitsPerWord - 1))
		{
			const int32 HiWordOffset = (BitCount - 1) / BitsPerWord;
			const int32 LoWordOffset = HiWordOffset + 1;
			const int32 HiWordShift  = BitCount - HiWordOffset * BitsPerWord;
			const int32 LoWordShift  = BitsPerWord - HiWordShift;
			Result.Bits[0] |= Bits[HiWordOffset] >> HiWordShift;
			for (int32 WordIndex = LoWordOffset; WordIndex < NumWords; ++WordIndex)
			{
				uint32 Value = Bits[WordIndex];
				Result.Bits[WordIndex - HiWordOffset] |= Value >> HiWordShift;
				Result.Bits[WordIndex - LoWordOffset] |= Value << LoWordShift;
			}
		}
		else
		{
			const int32 ShiftWords = BitCount / BitsPerWord;
			for (int32 WordIndex = NumWords - 1; WordIndex >= ShiftWords; --WordIndex)
			{
				Result.Bits[WordIndex - ShiftWords] = Bits[WordIndex];
			}
		}
		*this = Result;
	}

	/**
	 * Shift right by 1 bit.
	 */
	void ShiftRightByOneInternal()
	{
		const int32 LoWordShift = BitsPerWord - 1;
		Bits[0] = Bits[0] >> 1;
		for (int32 WordIndex = 1; WordIndex < NumWords; ++WordIndex)
		{
			const uint32 Value = Bits[WordIndex];
			Bits[WordIndex - 0] = Value >> 1;
			Bits[WordIndex - 1] |= Value << LoWordShift;
		}
	}

	/**
	 * Adds two integers.
	 */
	void Add(const BigInt& Other)
	{
		int64 CarryOver = 0;
		for (int32 WordIndex = 0; WordIndex < NumWords; ++WordIndex)
		{
			int64 WordSum = (int64)Other.Bits[WordIndex] + (int64)Bits[WordIndex] + CarryOver;
			CarryOver = WordSum >> BitsPerWord;
			WordSum &= 0xffffffff;
			Bits[WordIndex] = (uint32)WordSum;
		}
	}

	/**
	 * Subtracts two integers.
	 */
	void Subtract(const BigInt& Other)
	{
		BigInt NegativeOther(Other);
		NegativeOther.Negate();
		Add(NegativeOther);
	}

	/**
	 * Checks if this integer is negative.
	 */
	[[nodiscard]] bool IsNegative() const
	{
		if (bSigned)
		{
			return !!(Bits[NumWords - 1] & (1U << (BitsPerWord - 1)));
		}
		else
		{
			return false;
		}
	}

	/**
	 * Returns the sign of this integer.
	 */
	[[nodiscard]] int32 Sign() const
	{
		return IsNegative() ? -1 : 1;
	}

	/**
	 * Negates this integer. value = -value
	 */
	void Negate()
	{
		for (int32 WordIndex = 0; WordIndex < NumWords; ++WordIndex)
		{
			Bits[WordIndex] = ~Bits[WordIndex];
		}
		Add(One);
	}

	/**
	* Returns the index of the highest word that is not zero. -1 if no such word exists.
	*/
	[[nodiscard]] int32 GetHighestNonZeroWord() const
	{
		int32 WordIndex;
		for (WordIndex = NumWords - 1; WordIndex >= 0 && Bits[WordIndex] == 0; --WordIndex);
		return WordIndex;
	}

	/**
	* Multiplies two positive integers.
	 */
	void MultiplyFast(const BigInt& Factor)
	{
		const int64 Base = 2LL << BitsPerWord;
		TBigInt<NumBits * 2, bSigned> Result; // Twice as many bits for overflow protection
		uint32* ResultBits = Result.GetBits();
		BigInt Temp = *this;
		
		const int32 NumWordsA = Temp.GetHighestNonZeroWord() + 1;
		const int32 NumWordsB = Factor.GetHighestNonZeroWord() + 1;

		for (int32 WordIndexA = 0; WordIndexA < NumWordsA; ++WordIndexA)
		{
			const uint64 A = Temp.Bits[WordIndexA];
			uint64 Carry = 0;
			for (int32 WordIndexB = 0; WordIndexB < NumWordsB; ++WordIndexB)
			{
				const uint64 B = Factor.Bits[WordIndexB];
				const uint64 Product = ResultBits[WordIndexA + WordIndexB] + Carry + A * B;
				Carry = Product >> BitsPerWord;
				ResultBits[WordIndexA + WordIndexB] = (uint32)(Product & (Base - 1));
			}
			ResultBits[WordIndexA + NumWordsB] += (uint32)Carry;
		}

		for (int32 WordIndex = 0; WordIndex < NumWords; ++WordIndex)
		{
			Bits[WordIndex] = ResultBits[WordIndex];
		}
	}

	/**
	 * Multiplies two integers.
	 */
	void Multiply(const BigInt& Factor)
	{
		BigInt Result = *this;
		BigInt Other = Factor;
		
		int32 ResultSign;
		int32 OtherSign;
		MakePositiveFactors(Result, ResultSign, Other, OtherSign);

		Result.MultiplyFast(Other);

		// Restore the sign if necessary		
		RestoreSign(Result, OtherSign, ResultSign);
		*this = Result;		
	}

	/**
	 * Divides two integers with remainder.
	 */
	void DivideWithRemainder(const BigInt& Divisor, BigInt& Remainder) 
	{ 
		BigInt Denominator(Divisor);
		BigInt Dividend(*this);		
		
		int32 DenominatorSign;
		int32 DividendSign;
		MakePositiveFactors(Denominator, DenominatorSign, Dividend, DividendSign);

		if (Denominator > Dividend)
		{
			Remainder = *this;
			Zero();
			return;
		}
		if (Denominator == Dividend)
		{
			Remainder.Zero();
			*this = One;
			RestoreSign(*this, DenominatorSign, DividendSign);
			return;
		}

		BigInt Current(One);
		BigInt Quotient;

		int32 ShiftCount = Dividend.GetHighestNonZeroBit() - Denominator.GetHighestNonZeroBit();

		if (ShiftCount > 0)
		{
			Denominator.ShiftLeftInternal(ShiftCount);
		}

		while (Denominator <= Dividend) 
		{
			Denominator.ShiftLeftByOneInternal();
			ShiftCount++;			
		}

		Denominator.ShiftRightByOneInternal();

		ShiftCount--; // equivalent of a shift right
		if (ShiftCount)
		{
			Current.ShiftLeftInternal(ShiftCount);
		}

		// Reuse ShiftCount to track number of pending shifts
		ShiftCount = 0;
		const int32 NumLoops = Current.GetHighestNonZeroBit() + 1;

		for (int32 i = 0; i < NumLoops; ++i)
		{
			if (Dividend >= Denominator) 
			{
				if (ShiftCount != 0)
				{
					Current.ShiftRightInternal(ShiftCount);
					ShiftCount = 0;
				}

				Dividend -= Denominator;
				Quotient |= Current;
			}
			Denominator.ShiftRightByOneInternal();
			ShiftCount++;
		}    
		RestoreSign(Quotient, DenominatorSign, DividendSign);
		Remainder = Dividend;
		*this = Quotient;
	}

	/**
	 * Divides two integers.
	 */
	void Divide(const BigInt& Divisor) 
	{
		BigInt Remainder;
		DivideWithRemainder(Divisor, Remainder);
	}

	/**
	 * Performs modulo operation on this integer.
	 */
	void Modulo(const BigInt& Modulus)
	{
		// a mod b = a - floor(a/b)*b
		check(!IsNegative());
		BigInt Dividend(*this);

		// Dividend.Divide(Modulus);
		BigInt Temp;
		Dividend.DivideWithRemainder(Modulus, Temp);
		Dividend.MultiplyFast(Modulus);

		// Subtract(Dividend);
		Dividend.Negate();
		Add(Dividend);
	}

	/**
	 * Calculates square root of this integer.
	 */
	void Sqrt() 
	{
		BigInt Number(*this);
    BigInt Result;

    BigInt Bit(1);
		Bit.ShiftLeftInternal(NumBits - 2);
    while (Bit > Number)
		{
			Bit.ShiftRightInternal(2);
		}
 
		BigInt Temp;
    while (!Bit.IsZero()) 
		{
			Temp = Result;
			Temp.Add(Bit);
			if (Number >= Temp) 
			{
				Number -= Temp;
				Result.ShiftRightInternal(1);
				Result += Bit;
			}
			else
			{
				Result.ShiftRightInternal(1);
			}
			Bit.ShiftRightInternal(2);
    }
    *this = Result;
	}

	/**
	 * Assignment operator for int64 values.
	 */
	TBigInt& operator=(int64 Other)
	{
		Set(Other);
		return *this;
	}

	/**
	 * Returns the index of the highest non-zero bit. -1 if no such bit exists.
	 */
	[[nodiscard]] int32 GetHighestNonZeroBit() const
	{
		for (int32 WordIndex = NumWords - 1; WordIndex >= 0; --WordIndex)
		{
			if (!!Bits[WordIndex])
			{
				int32 BitIndex;
				for (BitIndex = BitsPerWord - 1; BitIndex >= 0 && !(Bits[WordIndex] & (1 << BitIndex)); --BitIndex);
				return BitIndex + WordIndex * BitsPerWord;
			}
		}
		return -1;
	}

	/**
	 * Returns a bit value as an integer value (0 or 1).
	 */
	[[nodiscard]] int32 GetBit(int32 BitIndex) const
	{
		const int32 WordIndex = BitIndex / BitsPerWord;
		BitIndex -= WordIndex * BitsPerWord;
		return (Bits[WordIndex] & (1 << BitIndex)) ? 1 : 0;
	}

	/**
	 * Sets a bit value.
	 */
	void SetBit(int32 BitIndex, int32 Value)
	{
		const int32 WordIndex = BitIndex / BitsPerWord;
		BitIndex -= WordIndex * BitsPerWord;
		if (!!Value)
		{
			Bits[WordIndex] |= (1 << BitIndex);
		}
		else
		{
			Bits[WordIndex] &= ~(1 << BitIndex);
		}
	}

	/**
	 * Shift left by the specified amount of bits.
	 *
	 * @param BitCount the number of bits to shift.
	 */
	void ShiftLeft(const int32 BitCount)
	{
		// Early out in the trivial cases
		if (BitCount == 0)
		{
			return;
		}
		else if (BitCount < 0)
		{
			ShiftRight(-BitCount);
			return;
		}
		else if (BitCount >= NumBits)
		{
			Zero();
			return;
		}
		ShiftLeftInternal(BitCount);
	}

	/**
	 * Shift right by the specified amount of bits.
	 *
	 * @param BitCount the number of bits to shift.
	 */
	void ShiftRight(const int32 BitCount)
	{
		// Early out in the trivial cases
		if (BitCount == 0)
		{
			return;
		}
		else if (BitCount < 0)
		{
			ShiftLeft(-BitCount);
			return;
		}
		else if (BitCount >= NumBits)
		{
			Zero();
			return;
		}
		ShiftRightInternal(BitCount);
	}

	/**
	 * Bitwise 'or'
	 */
	void BitwiseOr(const BigInt& Other)
	{
		for (int32 WordIndex = 0; WordIndex < NumWords; ++WordIndex)
		{
			Bits[WordIndex] |= Other.Bits[WordIndex];
		}
	}

	/**
	 * Bitwise 'and'
	 */
	void BitwiseAnd(const BigInt& Other)
	{
		for (int32 WordIndex = 0; WordIndex < NumWords; ++WordIndex)
		{
			Bits[WordIndex] &= Other.Bits[WordIndex];
		}
	}

	/**
	 * Bitwise 'not'
	 */
	void BitwiseNot()
	{
		for (int32 WordIndex = 0; WordIndex < NumWords; ++WordIndex)
		{
			Bits[WordIndex] = ~Bits[WordIndex];
		}
	}

	/**
	 * Checks if two integers are equal.
	 */
	[[nodiscard]] bool IsEqual(const BigInt& Other) const
	{
		for (int32 WordIndex = 0; WordIndex < NumWords; ++WordIndex)
		{
			if (Bits[WordIndex] != Other.Bits[WordIndex])
			{
				return false;
			}
		}

		return true;
	}

	/**
	 * this < Other
	 */
	[[nodiscard]] bool IsLess(const BigInt& Other) const
	{
		if (IsNegative())
		{
			if (!Other.IsNegative())
			{
				return true;
			}
			else
			{
				return IsGreater(Other);
			}
		}
		int32 WordIndex;
		for (WordIndex = NumWords - 1; WordIndex >= 0 && Other.Bits[WordIndex] == Bits[WordIndex]; --WordIndex);
		return WordIndex >= 0 && Bits[WordIndex] < Other.Bits[WordIndex];
	}

	/**
	 * this <= Other
	 */
	[[nodiscard]] bool IsLessOrEqual(const BigInt& Other) const
	{
		if (IsNegative())
		{
			if (!Other.IsNegative())
			{
				return true;
			}
			else
			{
				return IsGreaterOrEqual(Other);
			}
		}
		int32 WordIndex;
		for (WordIndex = NumWords - 1; WordIndex >= 0 && Other.Bits[WordIndex] == Bits[WordIndex]; --WordIndex);
		return WordIndex < 0 || Bits[WordIndex] < Other.Bits[WordIndex];
	}

	/**
	 * this > Other
	 */
	[[nodiscard]] bool IsGreater(const BigInt& Other) const
	{
		if (IsNegative())
		{
			if (!Other.IsNegative())
			{
				return false;
			}
			else
			{
				return IsLess(Other);
			}
		}
		int32 WordIndex;
		for (WordIndex = NumWords - 1; WordIndex >= 0 && Other.Bits[WordIndex] == Bits[WordIndex]; --WordIndex);
		return WordIndex >= 0 && Bits[WordIndex] > Other.Bits[WordIndex];
	}

	/**
	 * this >= Other
	 */
	[[nodiscard]] bool IsGreaterOrEqual(const BigInt& Other) const
	{
		if (IsNegative())
		{
			if (Other.IsNegative())
			{
				return false;
			}
			else
			{
				return IsLessOrEqual(Other);
			}
		}
		int32 WordIndex;
		for (WordIndex = NumWords - 1; WordIndex >= 0 && Other.Bits[WordIndex] == Bits[WordIndex]; --WordIndex);
		return WordIndex < 0 || Bits[WordIndex] > Other.Bits[WordIndex];
	}

	/**
	 * this == 0
	 */
	[[nodiscard]] bool IsZero() const
	{
		int32 WordIndex;
		for (WordIndex = NumWords - 1; WordIndex >= 0 && !Bits[WordIndex]; --WordIndex);
		return WordIndex < 0;
	}

	/**
	 * this > 0
	 */
	[[nodiscard]] bool IsGreaterThanZero() const
	{
		return !IsNegative() && !IsZero();
	}

	/**
	 * this < 0
	 */
	[[nodiscard]] bool IsLessThanZero() const
	{
		return IsNegative() && !IsZero();
	}

	[[nodiscard]] bool IsFirstBitSet() const
	{
		return !!(Bits[0] & 1);
	}
	/**
	 * Bit indexing operator
	 */
	[[nodiscard]] bool operator[](int32 BitIndex) const
	{
		return GetBit(BitIndex);
	}

	// Begin operator overloads
	[[nodiscard]] BigInt operator>>(int32 Count) const
	{
		BigInt Result = *this;
		Result.ShiftRight(Count);
		return Result;
	}

	BigInt& operator>>=(int32 Count)
	{
		ShiftRight(Count);
		return *this;
	}

	[[nodiscard]] BigInt operator<<(int32 Count) const
	{
		BigInt Result = *this;
		Result.ShiftLeft(Count);
		return Result;
	}

	BigInt& operator<<=(int32 Count)
	{
		ShiftLeft(Count);
		return *this;
	}

	[[nodiscard]] BigInt operator+(const BigInt& Other) const
	{
		BigInt Result(*this);
		Result.Add(Other);
		return Result;
	}

	BigInt& operator++()
	{
		Add(One);
		return *this;
	}

	BigInt& operator+=(const BigInt& Other)
	{
		Add(Other);
		return *this;
	}

	[[nodiscard]] BigInt operator-(const BigInt& Other) const
	{
		BigInt Result(*this);
		Result.Subtract(Other);
		return Result;
	}

	BigInt& operator--()
	{
		Subtract(One);
		return *this;
	}

	BigInt& operator-=(const BigInt& Other)
	{
		Subtract(Other);
		return *this;
	}

	[[nodiscard]] BigInt operator*(const BigInt& Other) const
	{
		BigInt Result(*this);
		Result.Multiply(Other);
		return Result;
	}

	BigInt& operator*=(const BigInt& Other)
	{
		Multiply(Other);
		return *this;
	}

	[[nodiscard]] BigInt operator/(const BigInt& Divider) const
	{
		BigInt Result(*this);
		Result.Divide(Divider);
		return Result;
	}

	BigInt& operator/=(const BigInt& Divider)
	{
		Divide(Divider);
		return *this;
	}

	[[nodiscard]] BigInt operator%(const BigInt& Modulus) const
	{
		BigInt Result(*this);
		Result.Modulo(Modulus);
		return Result;
	}

	BigInt& operator%=(const BigInt& Modulus)
	{
		Modulo(Modulus);
		return *this;
	}

	[[nodiscard]] bool operator<(const BigInt& Other) const
	{
		return IsLess(Other);
	}

	[[nodiscard]] bool operator<=(const BigInt& Other) const
	{
		return IsLessOrEqual(Other);
	}

	[[nodiscard]] bool operator>(const BigInt& Other) const
	{
		return IsGreater(Other);
	}

	[[nodiscard]] bool operator>=(const BigInt& Other) const
	{
		return IsGreaterOrEqual(Other);
	}

	[[nodiscard]] bool operator==(const BigInt& Other) const
	{
		return IsEqual(Other);
	}

	[[nodiscard]] bool operator!=(const BigInt& Other) const
	{
		return !IsEqual(Other);
	}

	[[nodiscard]] BigInt operator&(const BigInt& Other) const
	{
		BigInt Result(*this);
		Result.BitwiseAnd(Other);
		return Result;
	}

	BigInt& operator&=(const BigInt& Other)
	{
		BitwiseAnd(Other);
		return *this;
	}

	[[nodiscard]] BigInt operator|(const BigInt& Other) const
	{
		BigInt Result(*this);
		Result.BitwiseOr(Other);
		return Result;
	}

	BigInt& operator|=(const BigInt& Other)
	{
		BitwiseOr(Other);
		return *this;
	}

	[[nodiscard]] BigInt operator~() const
	{
		BigInt Result(*this);
		Result.BitwiseNot();
		return Result;
	}
	// End operator overloads

	/**
	 * Returns the value of this big int as a 64-bit integer. If the value is greater, the higher bits are truncated.
	 */
	[[nodiscard]] int64 ToInt() const
	{
		return (int64)Bits[0] + ((int64)Bits[1] << BitsPerWord);
	}

	/**
	 * Returns this big int as a string.
	 */
	[[nodiscard]] FString ToString() const
	{
		FString Text("0x");
		int32 WordIndex;
		for (WordIndex = NumWords - 1; WordIndex > 0; --WordIndex)
		{
			if (Bits[WordIndex])
			{
				break;
			}
		}
		for (; WordIndex >= 0; --WordIndex)
		{
			Text += FString::Printf(TEXT("%08x"), Bits[WordIndex]);
		}
		return Text;
	}

	/**
	 * Parses a string representing a hex value
	 */
	void Parse(const FString& Value)
	{
		Zero();
		int32 ValueStartIndex = 0;
		if (Value.Len() >= 2 && Value[0] == '0' && FChar::ToUpper(Value[1]) == 'X')
		{
			ValueStartIndex = 2;
		}
		check((Value.Len() - ValueStartIndex) <= (NumBits / 4));
		const int32 NybblesPerWord = BitsPerWord / 4;
		for (int32 CharIndex = Value.Len() - 1; CharIndex >= ValueStartIndex; --CharIndex)
		{
			const TCHAR ByteChar = Value[CharIndex];
			const int32 ByteIndex = (Value.Len() - CharIndex - 1);
			const int32 WordIndex = ByteIndex / NybblesPerWord;
			const int32 ByteValue = ByteChar > '9' ? (FChar::ToUpper(ByteChar) - 'A' + 10) : (ByteChar - '0');
			check(ByteValue >= 0 && ByteValue <= 15);
			Bits[WordIndex] |= (ByteValue << (ByteIndex % NybblesPerWord) * 4);
		}
	}

	/**
	 * Serialization.
	 */
	friend FArchive& operator<<(FArchive& Ar, BigInt& Value)
	{
		for (int32 Index = 0; Index < NumWords; ++Index)
		{
			Ar << Value.Bits[Index];
		}
		return Ar;
	}
};

template<int32 NumBits, bool bSigned>
TBigInt<NumBits, bSigned> TBigInt<NumBits, bSigned>::One(1LL);

// Predefined big int types
typedef TBigInt<256> int256;
typedef TBigInt<512> int512;
typedef TBigInt<512> TEncryptionInt;

/**
 * Encryption key - exponent and modulus pair
 */
template <typename IntType>
struct TEncryptionKey
{
	IntType Exponent;
	IntType Modulus;
};
typedef TEncryptionKey<TEncryptionInt> FEncryptionKey;

/**
 * Math utils for encryption.
 */
namespace FEncryption
{
	/**
	 * Greatest common divisor of ValueA and ValueB.
	 */
	template <typename IntType>
	IntType CalculateGCD(IntType ValueA, IntType ValueB)
	{ 
		// Early out in obvious cases
		if (ValueA == 0)
		{
			return ValueA;
		}
		if (ValueB == 0) 
		{
			return ValueB;
		}
		
		// Shift is log(n) where n is the greatest power of 2 dividing both A and B.
		int32 Shift;
		for (Shift = 0; ((ValueA | ValueB) & 1) == 0; ++Shift) 
		{
			ValueA >>= 1;
			ValueB >>= 1;
		}
 
		while ((ValueA & 1) == 0)
		{
			ValueA >>= 1;
		}
 
		do 
		{
			// Remove all factors of 2 in B.
			while ((ValueB & 1) == 0)
			{
				ValueB >>= 1;
			}
 
			// Make sure A <= B
			if (ValueA > ValueB) 
			{
				Swap(ValueA, ValueB);
			}
			ValueB = ValueB - ValueA;
		} 
		while (ValueB != 0);
 
		// Restore common factors of 2.
		return ValueA << Shift;
	}

	/** 
	 * Multiplicative inverse of exponent using extended GCD algorithm.
	 *
	 * Extended gcd: ax + by = gcd(a, b), where a = exponent, b = fi(n), gcd(a, b) = gcd(e, fi(n)) = 1, fi(n) is the Euler's totient function of n
	 * We only care to find d = x, which is our multiplicatve inverse of e (a).
	 */
	template <typename IntType>
	IntType CalculateMultiplicativeInverseOfExponent(IntType Exponent, IntType Totient)
	{
		IntType x0 = 1;
		IntType y0 = 0;
		IntType x1 = 0;
		IntType y1 = 1;
		IntType a0 = Exponent;
		IntType b0 = Totient;
		while (b0 != 0)
		{
			// Quotient = Exponent / Totient
			IntType Quotient = a0 / b0;

			// (Exponent, Totient) = (Totient, Exponent mod Totient)
			IntType b1 = a0 % b0;
			a0 = b0;
			b0 = b1;

			// (x, lastx) = (lastx - Quotient*x, x)
			IntType x2 = x0 - Quotient * x1;
			x0 = x1;
			x1 = x2;

			// (y, lasty) = (lasty - Quotient*y, y)
			IntType y2 = y0 - Quotient * y1;
			y0 = y1;
			y1 = y2;
		}
		// If x0 is negative, find the corresponding positive element in (mod Totient)
		while (x0 < 0)
		{
			x0 += Totient;
		}
		return x0;
	}

	/**
	 * Generate Key Pair for encryption and decryption.
	 */
	template <typename IntType>
	void GenerateKeyPair(const IntType& P, const IntType& Q, FEncryptionKey& PublicKey, FEncryptionKey& PrivateKey)
	{
		const IntType Modulus = P * Q;
		const IntType Fi = (P - 1) * (Q - 1);

		IntType EncodeExponent = Fi;
		IntType DecodeExponent = 0;
		do
		{
			for ( --EncodeExponent; EncodeExponent > 1 && CalculateGCD(EncodeExponent, Fi) > 1; --EncodeExponent);
			DecodeExponent = CalculateMultiplicativeInverseOfExponent(EncodeExponent, Fi);
		}
		while (DecodeExponent == EncodeExponent);

		PublicKey.Exponent = DecodeExponent;
		PublicKey.Modulus = Modulus;

		PrivateKey.Exponent = EncodeExponent;
		PrivateKey.Modulus = Modulus;
	}

	/** 
	 * Raise Base to power of Exponent in mod Modulus.
	 */
	template <typename IntType>
	IntType ModularPow(IntType Base, IntType Exponent, IntType Modulus)
	{
		const IntType One(1);
		IntType Result(1);
		while (Exponent > 0)
		{
			if (Exponent.IsFirstBitSet())
			{
				Result = (Result * Base) % Modulus;
			}
			Exponent >>= 1;
			Base = (Base * Base) % Modulus;
		}
		return Result;
	}

	/**
	* Encrypts a stream of bytes
	*/
	template <typename IntType>
	void EncryptBytes(IntType* EncryptedData, const uint8* Data, const int64 DataLength, const FEncryptionKey& EncryptionKey)
	{
		for (int Index = 0; Index < DataLength; ++Index)
		{
			EncryptedData[Index] = FEncryption::ModularPow(IntType(Data[Index]), EncryptionKey.Exponent, EncryptionKey.Modulus);
		}
	}

	/**
	* Decrypts a stream of bytes
	*/
	template <typename IntType>
	void DecryptBytes(uint8* DecryptedData, const IntType* Data, const int64 DataLength, const FEncryptionKey& DecryptionKey)
	{
		for (int64 Index = 0; Index < DataLength; ++Index)
		{
			IntType DecryptedByte = FEncryption::ModularPow(Data[Index], DecryptionKey.Exponent, DecryptionKey.Modulus);
			DecryptedData[Index] = (uint8)DecryptedByte.ToInt();
		}
	}
}

/// @cond DOXYGEN_WARNINGS

/**
 * Specialization for int type used in encryption (performance). Avoids using temporary results and most of the operations are inplace.
 */
template <>
inline TEncryptionInt FEncryption::ModularPow(TEncryptionInt Base, TEncryptionInt Exponent, TEncryptionInt Modulus)
{
	TEncryptionInt Result(1LL);
	while (Exponent.IsGreaterThanZero())
	{
		if (Exponent.IsFirstBitSet())
		{
			Result.MultiplyFast(Base);
			Result.Modulo(Modulus);
		}
		Exponent.ShiftRightByOneInternal();
		Base.MultiplyFast(Base);
		Base.Modulo(Modulus);
	}
	return Result;
}

/// @endcond

template <class InDataType>
struct FSignatureBase
{
	typedef InDataType DataType;

	DataType Data;

	FSignatureBase()
	{
		Data = 0;
	}

	void Serialize(FArchive& Ar)
	{
		Ar << Data;
	}

	static int64 Size()
	{
		return sizeof(InDataType);
	}

	bool IsValid() const
	{
		return Data != 0;
	}

	bool operator != (const FSignatureBase& InOther)
	{
		return Data != InOther.Data;
	}

	bool operator == (const FSignatureBase& InOther)
	{
		return Data == InOther.Data;
	}

	/**
	* Serialization.
	*/
	friend FArchive& operator<<(FArchive& Ar, FSignatureBase& Value)
	{
		Value.Serialize(Ar);
		return Ar;
	}
};

struct FEncryptedSignature : public FSignatureBase<TEncryptionInt>
{

};

struct FDecryptedSignature : public FSignatureBase<uint32>
{

};

namespace FEncryption
{
	inline void EncryptSignature(const FDecryptedSignature& InUnencryptedSignature, FEncryptedSignature& OutEncryptedSignature, const FEncryptionKey& EncryptionKey)
	{
		OutEncryptedSignature.Data = FEncryption::ModularPow(TEncryptionInt(InUnencryptedSignature.Data), EncryptionKey.Exponent, EncryptionKey.Modulus);
	}

	inline void DecryptSignature(const FEncryptedSignature& InEncryptedSignature, FDecryptedSignature& OutUnencryptedSignature, const FEncryptionKey& EncryptionKey)
	{
		OutUnencryptedSignature.Data = (FDecryptedSignature::DataType)FEncryption::ModularPow(TEncryptionInt(InEncryptedSignature.Data), EncryptionKey.Exponent, EncryptionKey.Modulus).ToInt();
	}
}

