// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"

template< typename T >
struct TBounds
{
	template< typename U > using TVector  = UE::Math::TVector<U>;
	template< typename U > using TVector4 = UE::Math::TVector4<U>;
	template< typename U > using TMatrix  = UE::Math::TMatrix<U>;
	using FReal = T;

	TVector4<T>	Min = TVector4<T>(  TNumericLimits<T>::Max(),  TNumericLimits<T>::Max(),  TNumericLimits<T>::Max() );
	TVector4<T>	Max = TVector4<T>( -TNumericLimits<T>::Max(), -TNumericLimits<T>::Max(), -TNumericLimits<T>::Max() );

	inline TBounds<T>& operator=( const TVector<T>& Other )
	{
		Min = Other;
		Max = Other;
		return *this;
	}

	inline TBounds<T>& operator+=( const TVector<T>& Other )
	{
		VectorStoreAligned( VectorMin( VectorLoadAligned( &Min ), VectorLoadFloat3( &Other ) ), &Min );
		VectorStoreAligned( VectorMax( VectorLoadAligned( &Max ), VectorLoadFloat3( &Other ) ), &Max );
		return *this;
	}

	inline TBounds<T>& operator+=( const TBounds<T>& Other )
	{
		VectorStoreAligned( VectorMin( VectorLoadAligned( &Min ), VectorLoadAligned( &Other.Min ) ), &Min );
		VectorStoreAligned( VectorMax( VectorLoadAligned( &Max ), VectorLoadAligned( &Other.Max ) ), &Max );
		return *this;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT TBounds<T> operator+( const TBounds<T>& Other ) const
	{
		return TBounds<T>(*this) += Other;
	}

	[[nodiscard]] inline bool Intersect( const TBounds<T>& Other ) const
	{
		auto MinVsOtherMax = VectorCompareGT(VectorLoadAligned(&Min), VectorLoadAligned(&Other.Max));
		auto OtherMinVsMax = VectorCompareGT(VectorLoadAligned(&Other.Min), VectorLoadAligned(&Max));
		int SeparatedMask = VectorMaskBits(VectorBitwiseOr(MinVsOtherMax, OtherMinVsMax));
		return (SeparatedMask & 7) == 0;
	}

	[[nodiscard]] inline bool Contains( const TBounds<T>& Other ) const
	{
		auto MinVsOtherMin = VectorCompareLE(VectorLoadAligned(&Min), VectorLoadAligned(&Other.Min));
		auto MaxVsOtherMax = VectorCompareGE(VectorLoadAligned(&Max), VectorLoadAligned(&Other.Max));
		int ContainedMask = VectorMaskBits(VectorBitwiseAnd(MinVsOtherMin, MaxVsOtherMax));
		return (ContainedMask & 7) == 7;
	}

	[[nodiscard]] inline T DistSqr( const TVector<T>& Point ) const
	{
		auto rMin		= VectorLoadAligned( &Min );
		auto rMax		= VectorLoadAligned( &Max );
		auto rPoint		= VectorLoadFloat3( &Point );
		auto rClosest	= VectorSubtract( VectorMin( VectorMax( rPoint, rMin ), rMax ), rPoint );
		return VectorDot3Scalar( rClosest, rClosest );
	}

	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> GetCenter() const
	{
		return (Max + Min) * 0.5f;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> GetExtent() const
	{
		return (Max - Min) * 0.5f;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> GetSize() const
	{
		return (Max - Min);
	}

	[[nodiscard]] inline T GetSurfaceArea() const
	{
		TVector<T> Size = Max - Min;
		return 0.5f * (Size.X * Size.Y + Size.X * Size.Z + Size.Y * Size.Z);
	}

	template< typename U >
	[[nodiscard]] inline auto ToAbsolute( const TVector<U>& Offset ) const
	{
		using CommonType = typename std::common_type<T, U>::type;

		TBounds< CommonType > Bounds;
		Bounds.Min = TVector4< CommonType >( Min ) + TVector4< CommonType >( Offset, 0.0 );
		Bounds.Max = TVector4< CommonType >( Max ) + TVector4< CommonType >( Offset, 0.0 );
		return Bounds;
	}

	template< typename U >
	[[nodiscard]] inline TBounds< float > ToRelative( const TVector<U>& Offset ) const
	{
		using CommonType = typename std::common_type<T, U>::type;

		TBounds< float > Bounds;
		Bounds.Min = TVector4< float >( TVector4< CommonType >( Min ) - TVector4< CommonType >( Offset, 0.0 ) );
		Bounds.Max = TVector4< float >( TVector4< CommonType >( Max ) - TVector4< CommonType >( Offset, 0.0 ) );
		return Bounds;
	}

	inline TBounds<T> TransformBy( const TMatrix<T>& M ) const
	{
		auto rMin		= VectorLoadAligned( &Min );
		auto rMax		= VectorLoadAligned( &Max );

		auto m0			= VectorLoadAligned( M.M[0] );
		auto m1			= VectorLoadAligned( M.M[1] );
		auto m2			= VectorLoadAligned( M.M[2] );
		auto m3			= VectorLoadAligned( M.M[3] );

		auto Half		= VectorSetFloat1( (T)0.5f );
		auto Origin		= VectorMultiply( VectorAdd(		rMax, rMin ), Half );
		auto Extent		= VectorMultiply( VectorSubtract(	rMax, rMin ), Half );

		auto NewOrigin	= VectorMultiplyAdd( VectorReplicate( Origin, 0 ), m0, m3 );
		NewOrigin		= VectorMultiplyAdd( VectorReplicate( Origin, 1 ), m1, NewOrigin );
		NewOrigin		= VectorMultiplyAdd( VectorReplicate( Origin, 2 ), m2, NewOrigin );

		auto NewExtent	=						VectorAbs( VectorMultiply( VectorReplicate( Extent, 0 ), m0 ) );
		NewExtent		= VectorAdd( NewExtent,	VectorAbs( VectorMultiply( VectorReplicate( Extent, 1 ), m1 ) ) );
		NewExtent		= VectorAdd( NewExtent,	VectorAbs( VectorMultiply( VectorReplicate( Extent, 2 ), m2 ) ) );

		auto NewMin		= VectorSubtract(	NewOrigin, NewExtent );
		auto NewMax		= VectorAdd(		NewOrigin, NewExtent );

		TBounds<T> Bounds;
		VectorStoreAligned( NewMin, &Bounds.Min );
		VectorStoreAligned( NewMax, &Bounds.Max );
		return Bounds;
	}

	inline friend FArchive& operator<<( FArchive& Ar, TBounds<T>& Bounds )
	{
		Ar << Bounds.Min;
		Ar << Bounds.Max;
		return Ar;
	}
};

using FBounds3f = TBounds< float >;
using FBounds3d = TBounds< double >;
