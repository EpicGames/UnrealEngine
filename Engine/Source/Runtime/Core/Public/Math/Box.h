// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Math/MathFwd.h" // IWYU pragma: export
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Math/Vector.h"
#include "Math/Sphere.h"
#include "Misc/LargeWorldCoordinatesSerializer.h"

#ifndef ENABLE_UNINITIALIZED_BOX_DIAGNOSTIC
#define ENABLE_UNINITIALIZED_BOX_DIAGNOSTIC 0
#endif // ENABLE_UNINITIALIZED_BOX_DIAGNOSTIC

#ifndef ENABLE_BOX_DIAGNOSTIC_CHECKS
#define ENABLE_BOX_DIAGNOSTIC_CHECKS (ENABLE_UNINITIALIZED_BOX_DIAGNOSTIC || ENABLE_NAN_DIAGNOSTIC)
#endif// ENABLE_BOX_DIAGNOSTIC_CHECKS

/**
 * Implements an axis-aligned box.
 *
 * Boxes describe an axis-aligned extent in three dimensions. They are used for many different things in the
 * Engine and in games, such as bounding volumes, collision detection and visibility calculation.
 */
namespace UE::Math
{

struct TBoxConstInit {};

template<typename T>
struct TBox
{
	using FReal = T;

	/** Holds the box's minimum point. */
	TVector<T> Min;

	/** Holds the box's maximum point. */
	TVector<T> Max;

	/** Holds a flag indicating whether this box is valid. */
	uint8 IsValid;

private:
	uint8 Pad[sizeof(T) - sizeof(IsValid)];

public:

	/**
	 * Default constructor.
	 * Creates a new box with uninitialized extents and marks it as invalid.
	 *
	 * NOTE: This default constructor is unlike TVector, TMatrix etc. in that it
	 *       actually initializes the instance.  Ideally it should be = default;
	 *       in the same way, but this would break backwards compatibility.  It
	 *       might be possible if the questionable IsValid field ever gets removed.
	 */
	[[nodiscard]] constexpr TBox()
#if ENABLE_UNINITIALIZED_BOX_DIAGNOSTIC
		: Min(std::numeric_limits<T>::quiet_NaN(), TVectorConstInit{})
		, Max(std::numeric_limits<T>::quiet_NaN(), TVectorConstInit{})
		, IsValid(0)
#else
		// To be consistent with other engine core types (i.e. Vector) the other members are not initialized
		// since there is not meaningful default values.
		: IsValid(0)
#endif
	{
	}

	/** Creates a new box without initialization. */
	[[nodiscard]] explicit constexpr TBox(ENoInit)
	{
	}

	/** Creates and initializes a new box with zero extent and marks it as invalid. */
	[[nodiscard]] explicit constexpr TBox(EForceInit)
		: Min(0, TVectorConstInit{})
		, Max(0, TVectorConstInit{})
		, IsValid(0)
		, Pad() // For deterministic cooking with UPS serialize-as-zero for immutable/atomic noexport types
	{
	}

	[[nodiscard]] constexpr TBox(EForceInit, TBoxConstInit)
		: TBox(ForceInit)
	{
	}

#if ENABLE_BOX_DIAGNOSTIC_CHECKS
	inline void DiagnosticCheck() const
	{
		if (ContainsNaN())
		{
#if ENABLE_UNINITIALIZED_BOX_DIAGNOSTIC
			// In case ENABLE_UNINITIALIZED_BOX_DIAGNOSTIC was explicitly enabled then report each encountered accesses to uninitialized members.
			ensureAlwaysMsgf(false, TEXT("FBox contains NaN: %s."), *ToString());
#else
			// In case diagnostic check is enable from NaN diagnostic we report the error using the NaN error reporting behavior.
			logOrEnsureNanError(TEXT("FBox contains NaN: %s."), *ToString());
#endif
		}
	}
#else
	UE_FORCEINLINE_HINT void DiagnosticCheck() const {}
#endif

	[[nodiscard]] bool ContainsNaN() const
	{
		return Min.ContainsNaN() || Max.ContainsNaN();
	}
	
	/**
	 * Creates and initializes a new box from the specified extents.
	 *
	 * @param InMin The box's minimum point.
	 * @param InMax The box's maximum point.
	 */
	template<typename FArg>
	TBox(const TVector<FArg>& InMin, const TVector<FArg>& InMax)
		: Min(InMin)
		, Max(InMax)
		, IsValid(1)
	{
		// Intended to catch TBox<float>(TVector<double>(), TVector<double>())
		static_assert(sizeof(FArg) <= sizeof(T), "Losing precision when constructing a box of floats from vectors of doubles");

		DiagnosticCheck();
	}

	TBox(const TVector4<T>& InMin, const TVector4<T>& InMax)
		: Min(InMin)
		, Max(InMax)
		, IsValid(1)
	{
		DiagnosticCheck();
	}

	/**
	 * Creates and initializes a new box from the given set of points.
	 *
	 * @param Points Array of Points to create for the bounding volume.
	 * @param Count The number of points.
	 */
	TBox(const TVector<T>* Points, const int32 Count) : TBox(ForceInit)
	{
		for (int32 i = 0; i < Count; i++)
		{
			*this += Points[i];
		}
	}

	/**
	 * Creates and initializes a new box from an array of points.
	 *
	 * @param Points Array of Points to create for the bounding volume.
	 */
	explicit TBox(const TArray<TVector<T>>& Points) : TBox<T>(&Points[0], Points.Num()) {};

	// Conversion from other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TBox(const TBox<FArg>& From)
		: TBox<T>(TVector<T>(From.Min), TVector<T>(From.Max))
	{
	}

	/**
	 * Creates and initializes a box that tightly encloses the given sphere.
	 *
	 * @param Sphere The sphere being enclosed.
	 */
	explicit TBox(const TSphere<T>& Sphere) : TBox<T>(Sphere.Center - TVector<T>(Sphere.W), Sphere.Center + TVector<T>(Sphere.W)) {};

public:

	/**
	 * Compares two boxes for equality.
	 * 
	 * Returns true if both bounding boxes are invalid. Returns false if one of the bounding boxes is invalid.
	 *
	 * @return true if the boxes are equal, false otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==( const TBox<T>& Other ) const
	{
		return (!IsValid && !Other.IsValid) || ((IsValid && Other.IsValid) && (Min == Other.Min) && (Max == Other.Max));
	}

	/**
	 * Compares two boxes for inequality.
	 * 
	 * @return false if the boxes are equal, true otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=( const TBox<T>& Other) const
	{
		return !(*this == Other);
	}

	/**
	 * Check against another box for equality, within specified error limits.
	 * 
	 * Returns true if both bounding boxes are invalid. Returns false if one of the bounding boxes is invalid.
	 *
	 * @param Other The box to check against.
	 * @param Tolerance Error tolerance.
	 * @return true if the boxes are equal within tolerance limits, false otherwise.
	 */
	[[nodiscard]] bool Equals(const TBox<T>& Other, T Tolerance=UE_KINDA_SMALL_NUMBER) const
	{
		return (!IsValid && !Other.IsValid) || ((IsValid && Other.IsValid) && Min.Equals(Other.Min, Tolerance) && Max.Equals(Other.Max, Tolerance));
	}

	/**
	 * Adds to this bounding box to include a given point.
	 *
	 * @param Other the point to increase the bounding volume to.
	 * @return Reference to this bounding box after resizing to include the other point.
	 */
	inline TBox<T>& operator+=( const TVector<T> &Other );

	/**
	 * Gets the result of addition to this bounding volume.
	 *
	 * @param Other The other point to add to this.
	 * @return A new bounding volume.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TBox<T> operator+( const TVector<T>& Other ) const
	{
		return TBox<T>(*this) += Other;
	}

	/**
	 * Adds to this bounding box to include a new bounding volume.
	 *
	 * @param Other the bounding volume to increase the bounding volume to.
	 * @return Reference to this bounding volume after resizing to include the other bounding volume.
	 */
	inline TBox<T>& operator+=( const TBox<T>& Other );

	/**
	 * Gets the result of addition to this bounding volume.
	 *
	 * @param Other The other volume to add to this.
	 * @return A new bounding volume.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TBox<T> operator+( const TBox<T>& Other ) const
	{
		return TBox<T>(*this) += Other;
	}

	/**
	 * Gets reference to the min or max of this bounding volume.
	 *
	 * @param Index the index into points of the bounding volume.
	 * @return a reference to a point of the bounding volume.
	 */
    [[nodiscard]] inline TVector<T>& operator[]( int32 Index )
	{
		check((Index >= 0) && (Index < 2));

		if (Index == 0)
		{
			return Min;
		}

		return Max;
	}

	/* Gets reference to the min or max of this bounding volume.
	 *
	 * @param Index the index into points of the bounding volume.
	 * @return a const reference to a point of the bounding volume.
	 */
    [[nodiscard]] inline const TVector<T>& operator[]( int32 Index ) const
	{
		check((Index >= 0) && (Index < 2));

		if (Index == 0)
		{
			return Min;
		}

		return Max;
	}

public:

	/** 
	 * Calculates the distance of a point to this box.
	 *
	 * @param Point The point.
	 * @return The distance.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T ComputeSquaredDistanceToPoint( const TVector<T>& Point ) const
	{
		return ComputeSquaredDistanceFromBoxToPoint(Min, Max, Point);
	}

	/**
	 * Calculates squared distance between two boxes.
	 */
	[[nodiscard]] inline T ComputeSquaredDistanceToBox(const TBox<T>& Box) const
	{
		TVector<T> AxisDistances = (GetCenter() - Box.GetCenter()).GetAbs() - (GetExtent() + Box.GetExtent());
		AxisDistances = TVector<T>::Max(AxisDistances, TVector<T>(0.0f, 0.0f, 0.0f));
		return TVector<T>::DotProduct(AxisDistances, AxisDistances);
	}

	/** 
	 * Returns a box of increased size.
	 *
	 * @param W The size to increase the volume by.
	 * @return A new bounding box.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TBox<T> ExpandBy(T W) const
	{
		return TBox<T>(Min - TVector<T>(W, W, W), Max + TVector<T>(W, W, W));
	}

	/**
	* Returns a box of increased size.
	*
	* @param V The size to increase the volume by.
	* @return A new bounding box.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TBox<T> ExpandBy(const TVector<T>& V) const
	{
		return TBox<T>(Min - V, Max + V);
	}

	/**
	* Returns a box of increased size.
	*
	* @param Neg The size to increase the volume by in the negative direction (positive values move the bounds outwards)
	* @param Pos The size to increase the volume by in the positive direction (positive values move the bounds outwards)
	* @return A new bounding box.
	*/
	[[nodiscard]] TBox<T> ExpandBy(const TVector<T>& Neg, const TVector<T>& Pos) const
	{
		return TBox<T>(Min - Neg, Max + Pos);
	}

	/** 
	 * Returns a box with its position shifted.
	 *
	 * @param Offset The vector to shift the box by.
	 * @return A new bounding box.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TBox<T> ShiftBy( const TVector<T>& Offset ) const
	{
		return TBox<T>(Min + Offset, Max + Offset);
	}

	/** 
	 * Returns a box with its center moved to the new destination.
	 *
	 * @param Destination The destination point to move center of box to.
	 * @return A new bounding box.
	 */
	[[nodiscard]] inline TBox<T> MoveTo( const TVector<T>& Destination ) const
	{
		const TVector<T> Offset = Destination - GetCenter();
		return TBox<T>(Min + Offset, Max + Offset);
	}

	/**
	 * Gets the center point of this box.
	 *
	 * @return The center point.
	 * @see GetCenterAndExtents, GetExtent, GetSize, GetVolume
	 */
	inline TVector<T> GetCenter() const
	{
		DiagnosticCheck();
		return TVector<T>((Min + Max) * 0.5f);
	}

	/**
	 * Gets the center and extents of this box.
	 *
	 * @param Center [out] Will contain the box center point.
	 * @param Extents [out] Will contain the extent around the center.
	 * @see GetCenter, GetExtent, GetSize, GetVolume
	 */
	inline void GetCenterAndExtents( TVector<T>& Center, TVector<T>& Extents ) const
	{
		Extents = GetExtent();
		Center = Min + Extents;
	}

	/**
	 * Calculates the closest point on or inside the box to a given point in space.
	 *
	 * @param Point The point in space.
	 * @return The closest point on or inside the box.
	 */
	inline TVector<T> GetClosestPointTo( const TVector<T>& Point ) const;

	/**
	 * Gets the (half) extents of this box.
	 *
	 * @return The box (half) extents.
	 * @see GetCenter, GetCenterAndExtents, GetSize, GetVolume
	 */
	inline TVector<T> GetExtent() const
	{
		DiagnosticCheck();
		return 0.5f * (Max - Min);
	}

	/**
	 * Gets the size of this box.
	 *
	 * @return The box size.
	 * @see GetCenter, GetCenterAndExtents, GetExtent, GetVolume
	 */
	inline TVector<T> GetSize() const
	{
		DiagnosticCheck();
		return (Max - Min);
	}

	/**
	 * Gets the volume of this box.
	 *
	 * @return The box volume.
	 * @see GetCenter, GetCenterAndExtents, GetExtent, GetSize
	 */
	inline T GetVolume() const
	{
		DiagnosticCheck();
		return (Max.X - Min.X) * (Max.Y - Min.Y) * (Max.Z - Min.Z);
	}

	/**
	 * Set the initial values of the bounding box to Zero.
	 */
	inline void Init()
	{
		Min = Max = TVector<T>::ZeroVector;
		IsValid = 0;
	}

	/**
	 * Checks whether the given bounding box intersects this bounding box.
	 *
	 * @param Other The bounding box to intersect with.
	 * @return true if the boxes intersect, false otherwise.
	 *
	 * @note  This function assumes boxes have closed bounds, i.e. boxes with
	 *        coincident borders on any edge will overlap.
	 */
	inline bool Intersect( const TBox<T>& Other ) const;

	/**
	 * Checks whether the given bounding box intersects this bounding box in the XY plane.
	 *
	 * @param Other The bounding box to test intersection.
	 * @return true if the boxes intersect in the XY Plane, false otherwise.
	 *
	 * @note  This function assumes boxes have closed bounds, i.e. boxes with
	 *        coincident borders on any edge will overlap.
	 */
	inline bool IntersectXY( const TBox<T>& Other ) const;

	/**
	 * Returns the overlap TBox<T> of two box
	 *
	 * @param Other The bounding box to test overlap
	 * @return the overlap box. It can be 0 if they don't overlap
	 */
	[[nodiscard]] TBox<T> Overlap( const TBox<T>& Other ) const;

	/**
	  * Gets a bounding volume transformed by an inverted TTransform<T> object.
	  *
	  * @param M The transformation object to perform the inversely transform this box with.
	  * @return	The transformed box.
	  */
	[[nodiscard]] TBox<T> InverseTransformBy( const TTransform<T>& M ) const;

	/** 
	 * Checks whether the given location is inside this box.
	 * 
	 * @param In The location to test for inside the bounding volume.
	 * @return true if location is inside this volume.
	 * @see IsInsideXY
	 *
	 * @note  This function assumes boxes have open bounds, i.e. points lying on the border of the box are not inside.
	 *        Use IsInsideOrOn to include borders in the test.
	 */
	inline bool IsInside( const TVector<T>& In ) const
	{
		DiagnosticCheck();
		return ((In.X > Min.X) && (In.X < Max.X) && (In.Y > Min.Y) && (In.Y < Max.Y) && (In.Z > Min.Z) && (In.Z < Max.Z));
	}

	/** 
	 * Checks whether the given location is inside or on this box.
	 * 
	 * @param In The location to test for inside or on the bounding volume.
	 * @return true if location is inside or on this volume.
	 * @see IsInsideOrOnXY
	 *
	 * @note  This function assumes boxes have closed bounds, i.e. points lying on the border of the box are inside.
	 *        Use IsInside to exclude borders from the test.
	 */
	inline bool IsInsideOrOn( const TVector<T>& In ) const
	{
		DiagnosticCheck();
		return ((In.X >= Min.X) && (In.X <= Max.X) && (In.Y >= Min.Y) && (In.Y <= Max.Y) && (In.Z >= Min.Z) && (In.Z <= Max.Z));
	}

	/** 
	 * Checks whether a given box is fully encapsulated by this box.
	 * 
	 * @param Other The box to test for encapsulation within the bounding volume.
	 * @return true if box is inside this volume.
	 * @see IsInsideXY
	 *
	 * @note  This function assumes boxes have open bounds, i.e. boxes with
	 *        coincident borders on any edge are not encapsulated.
	 */
	UE_FORCEINLINE_HINT bool IsInside( const TBox<T>& Other ) const
	{
		return (IsInside(Other.Min) && IsInside(Other.Max));
	}

	/** 
	 * Checks whether a given box is fully encapsulated by this box.
	 * 
	 * @param Other The box to test for encapsulation within the bounding volume.
	 * @return true if box is inside this volume.
	 * @see IsInsideXY
	 *
	 * @note  This function assumes boxes have closed bounds, i.e. boxes with
	 *        coincident borders on any edge are encapsulated.
	 */
	UE_FORCEINLINE_HINT bool IsInsideOrOn( const TBox<T>& Other ) const
	{
		return (IsInsideOrOn(Other.Min) && IsInsideOrOn(Other.Max));
	}

	/** 
	 * Checks whether the given location is inside this box in the XY plane.
	 * 
	 * @param In The location to test for inside the bounding box.
	 * @return true if location is inside this box in the XY plane.
	 * @see IsInside
	 *
	 * @note  This function assumes boxes have open bounds, i.e. points lying on the border of the box are not inside.
	 *        Use IsInsideOrOnXY to include borders in the test.
	 */
	inline bool IsInsideXY( const TVector<T>& In ) const
	{
		DiagnosticCheck();
		return ((In.X > Min.X) && (In.X < Max.X) && (In.Y > Min.Y) && (In.Y < Max.Y));
	}

	/**
	 * Checks whether the given location is inside or on this box in the XY plane.
	 *
	 * @param In The location to test for inside or on the bounding volume.
	 * @return true if location is inside or on this box in the XY plane.
	 * @see IsInsideOrOn
	 *
	 * @note  This function assumes boxes have closed bounds, i.e. points lying on the border of the box are not inside.
	 *        Use IsInsideXY to exclude borders from the test.
	 */
	inline bool IsInsideOrOnXY(const TVector<T>& In) const
	{
		DiagnosticCheck();
		return ((In.X >= Min.X) && (In.X <= Max.X) && (In.Y >= Min.Y) && (In.Y <= Max.Y));
	}

	/** 
	 * Checks whether the given box is fully encapsulated by this box in the XY plane.
	 * 
	 * @param Other The box to test for encapsulation within the bounding box.
	 * @return true if box is inside this box in the XY plane.
	 * @see IsInside
	 *
	 * @note  This function assumes boxes have open bounds, i.e. boxes with
	 *        coincident borders on any edge are not encapsulated.
	 */
	UE_FORCEINLINE_HINT bool IsInsideXY( const TBox<T>& Other ) const
	{
		return (IsInsideXY(Other.Min) && IsInsideXY(Other.Max));
	}

	/**
	 * Gets a bounding volume transformed by a matrix.
	 *
	 * @param M The matrix to transform by.
	 * @return The transformed box.
	 * @see TransformProjectBy
	 */
	[[nodiscard]] TBox<T> TransformBy( const TMatrix<T>& M ) const;

	/**
	 * Gets a bounding volume transformed by a TTransform<T> object.
	 *
	 * @param M The transformation object.
	 * @return The transformed box.
	 * @see TransformProjectBy
	 */
	[[nodiscard]] TBox<T> TransformBy( const TTransform<T>& M ) const;

	/** 
	 * Returns the current world bounding box transformed and projected to screen space
	 *
	 * @param ProjM The projection matrix.
	 * @return The transformed box.
	 * @see TransformBy
	 */
	[[nodiscard]] TBox<T> TransformProjectBy( const TMatrix<T>& ProjM ) const;

	/**
	 * Get a textual representation of this box.
	 *
	 * @return A string describing the box.
	 */
	FString ToString() const;

	/**
	 * Get a textual representation of this box, without min/max values if invalid.
	 *
	 * @return A string describing the box.
	 */
	FString ToCompactString() const;

	/**
	 * Get the vertices that make up this box.
	 * 
	 * 
	 */
	void GetVertices( TVector<T> (&Vertices)[8] ) const;

public:

	/** 
	 * Utility function to build an AABB from Origin and Extent 
	 *
	 * @param Origin The location of the bounding box.
	 * @param Extent Half size of the bounding box.
	 * @return A new axis-aligned bounding box.
	 */
	static TBox<T> BuildAABB( const TVector<T>& Origin, const TVector<T>& Extent )
	{
		TBox<T> NewBox(Origin - Extent, Origin + Extent);

		return NewBox;
	}

public:

	/**
	 * Serializes the bounding box.
	 *
	 * @param Ar The archive to serialize into.
	 * @param Box The box to serialize.
	 * @return Reference to the Archive after serialization.
	 */
	friend FArchive& operator<<( FArchive& Ar, TBox<T>& Box )
	{
		return Ar << Box.Min << Box.Max << Box.IsValid;
	}

	/**
	 * Serializes the bounding box.
	 *
	 * @param Slot The structured archive slot to serialize into.
	 * @param Box The box to serialize.
	 */
	friend void operator<<(FStructuredArchive::FSlot Slot, TBox<T>& Box)
	{
		FStructuredArchive::FRecord Record = Slot.EnterRecord();
		Record << SA_VALUE(TEXT("Min"), Box.Min) << SA_VALUE(TEXT("Max"), Box.Max) << SA_VALUE(TEXT("IsValid"), Box.IsValid);
	}

	bool Serialize( FArchive& Ar )
	{
		Ar << *this;
		return true;
	}

	bool Serialize( FStructuredArchive::FSlot Slot )
	{
		Slot << *this;
		return true;
	}

	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		if constexpr (std::is_same_v<T, float>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Box, Box3f, Box3d);
		}
		else if constexpr (std::is_same_v<T, double>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Box, Box3d, Box3f);
		}
		else
		{
			static_assert(sizeof(T) == 0, "Unimplemented");
			return false;
		}
	}
};

/* TBox<T> inline functions
 *****************************************************************************/

template<typename T>
inline TBox<T>& TBox<T>::operator+=( const TVector<T> &Other )
{
	if (IsValid)
	{
		DiagnosticCheck();
		// Not testing Other.DiagnosticCheck() since TVector is not initialized to NaN

		Min.X = FMath::Min(Min.X, Other.X);
		Min.Y = FMath::Min(Min.Y, Other.Y);
		Min.Z = FMath::Min(Min.Z, Other.Z);

		Max.X = FMath::Max(Max.X, Other.X);
		Max.Y = FMath::Max(Max.Y, Other.Y);
		Max.Z = FMath::Max(Max.Z, Other.Z);
	}
	else
	{
		Min = Max = Other;
		IsValid = 1;
	}

	return *this;
}


template<typename T>
inline TBox<T>& TBox<T>::operator+=( const TBox<T>& Other )
{
	if (IsValid && Other.IsValid)
	{
		DiagnosticCheck();
		// Testing Other.DiagnosticCheck() since TBox can be initialized to NaN
		Other.DiagnosticCheck();

		Min.X = FMath::Min(Min.X, Other.Min.X);
		Min.Y = FMath::Min(Min.Y, Other.Min.Y);
		Min.Z = FMath::Min(Min.Z, Other.Min.Z);

		Max.X = FMath::Max(Max.X, Other.Max.X);
		Max.Y = FMath::Max(Max.Y, Other.Max.Y);
		Max.Z = FMath::Max(Max.Z, Other.Max.Z);
	}
	else if (Other.IsValid)
	{
		*this = Other;
	}

	return *this;
}

template<typename T>
inline TVector<T> TBox<T>::GetClosestPointTo( const TVector<T>& Point ) const
{
	DiagnosticCheck();

	// start by considering the point inside the box
	TVector<T> ClosestPoint = Point;

	// now clamp to inside box if it's outside
	if (Point.X < Min.X)
	{
		ClosestPoint.X = Min.X;
	}
	else if (Point.X > Max.X)
	{
		ClosestPoint.X = Max.X;
	}

	// now clamp to inside box if it's outside
	if (Point.Y < Min.Y)
	{
		ClosestPoint.Y = Min.Y;
	}
	else if (Point.Y > Max.Y)
	{
		ClosestPoint.Y = Max.Y;
	}

	// Now clamp to inside box if it's outside.
	if (Point.Z < Min.Z)
	{
		ClosestPoint.Z = Min.Z;
	}
	else if (Point.Z > Max.Z)
	{
		ClosestPoint.Z = Max.Z;
	}

	return ClosestPoint;
}


template<typename T>
inline bool TBox<T>::Intersect( const TBox<T>& Other ) const
{
	DiagnosticCheck();
	// Testing Other.DiagnosticCheck() since TBox can be initialized to NaN
	Other.DiagnosticCheck();

	if ((Min.X > Other.Max.X) || (Other.Min.X > Max.X))
	{
		return false;
	}

	if ((Min.Y > Other.Max.Y) || (Other.Min.Y > Max.Y))
	{
		return false;
	}

	if ((Min.Z > Other.Max.Z) || (Other.Min.Z > Max.Z))
	{
		return false;
	}

	return true;
}


template<typename T>
inline bool TBox<T>::IntersectXY( const TBox<T>& Other ) const
{
	DiagnosticCheck();
	// Testing Other.DiagnosticCheckUninitialized() since TBox can be initialized to NaN
	Other.DiagnosticCheck();

	if ((Min.X > Other.Max.X) || (Other.Min.X > Max.X))
	{
		return false;
	}

	if ((Min.Y > Other.Max.Y) || (Other.Min.Y > Max.Y))
	{
		return false;
	}

	return true;
}


template<typename T>
UE_FORCEINLINE_HINT FString TBox<T>::ToString() const
{
	return FString::Printf(TEXT("IsValid=%s, Min=(%s), Max=(%s)"), IsValid ? TEXT("true") : TEXT("false"), *Min.ToString(), *Max.ToString());
}

template<typename T>
UE_FORCEINLINE_HINT FString TBox<T>::ToCompactString() const
{
	return IsValid ? FString::Printf(TEXT("Min=(%s), Max=(%s)"), *Min.ToString(), *Max.ToString()) : TEXT("IsValid=false");
}


template<typename T>
TBox<T> TBox<T>::TransformBy(const TMatrix<T>& M) const
{
	// if we are not valid, return another invalid box.
	if (!IsValid)
	{
		return TBox<T>(ForceInit);
	}

	DiagnosticCheck();

	TBox<T> NewBox;

	const TVectorRegisterType<T> VecMin = VectorLoadFloat3_W0(&Min);
	const TVectorRegisterType<T> VecMax = VectorLoadFloat3_W0(&Max);

	const TVectorRegisterType<T> m0 = VectorLoadAligned(M.M[0]);
	const TVectorRegisterType<T> m1 = VectorLoadAligned(M.M[1]);
	const TVectorRegisterType<T> m2 = VectorLoadAligned(M.M[2]);
	const TVectorRegisterType<T> m3 = VectorLoadAligned(M.M[3]);

	const TVectorRegisterType<T> Half = VectorSetFloat1((T)0.5f); // VectorSetFloat1() can be faster than SetFloat3(0.5, 0.5, 0.5, 0.0). Okay if 4th element is 0.5, it's multiplied by 0.0 below and we discard W anyway.
	const TVectorRegisterType<T> Origin = VectorMultiply(VectorAdd(VecMax, VecMin), Half);
	const TVectorRegisterType<T> Extent = VectorMultiply(VectorSubtract(VecMax, VecMin), Half);

	TVectorRegisterType<T> NewOrigin = VectorMultiply(VectorReplicate(Origin, 0), m0);
	NewOrigin = VectorMultiplyAdd(VectorReplicate(Origin, 1), m1, NewOrigin);
	NewOrigin = VectorMultiplyAdd(VectorReplicate(Origin, 2), m2, NewOrigin);
	NewOrigin = VectorAdd(NewOrigin, m3);

	TVectorRegisterType<T> NewExtent = VectorAbs(VectorMultiply(VectorReplicate(Extent, 0), m0));
	NewExtent = VectorAdd(NewExtent, VectorAbs(VectorMultiply(VectorReplicate(Extent, 1), m1)));
	NewExtent = VectorAdd(NewExtent, VectorAbs(VectorMultiply(VectorReplicate(Extent, 2), m2)));

	const TVectorRegisterType<T> NewVecMin = VectorSubtract(NewOrigin, NewExtent);
	const TVectorRegisterType<T> NewVecMax = VectorAdd(NewOrigin, NewExtent);

	VectorStoreFloat3(NewVecMin, &(NewBox.Min.X));
	VectorStoreFloat3(NewVecMax, &(NewBox.Max.X));

	NewBox.IsValid = 1;

	return NewBox;
}

template<typename T>
TBox<T> TBox<T>::TransformBy(const TTransform<T>& M) const
{
	return TransformBy(M.ToMatrixWithScale());
}

template<typename T>
void TBox<T>::GetVertices(TVector<T>(&Vertices)[8]) const
{
	DiagnosticCheck();

	Vertices[0] = TVector<T>(Min);
	Vertices[1] = TVector<T>(Min.X, Min.Y, Max.Z);
	Vertices[2] = TVector<T>(Min.X, Max.Y, Min.Z);
	Vertices[3] = TVector<T>(Max.X, Min.Y, Min.Z);
	Vertices[4] = TVector<T>(Max.X, Max.Y, Min.Z);
	Vertices[5] = TVector<T>(Max.X, Min.Y, Max.Z);
	Vertices[6] = TVector<T>(Min.X, Max.Y, Max.Z);
	Vertices[7] = TVector<T>(Max);
}

template<typename T>
TBox<T> TBox<T>::InverseTransformBy(const TTransform<T>& M) const
{
	// if we are not valid, return another invalid box.
	if (!IsValid)
	{
		return TBox<T>(ForceInit);
	}

	TVector<T> Vertices[8];
	GetVertices(Vertices);

	TBox<T> NewBox(ForceInit);

	for (int32 VertexIndex = 0; VertexIndex < UE_ARRAY_COUNT(Vertices); VertexIndex++)
	{
		TVector<T> ProjectedVertex = M.InverseTransformPosition(Vertices[VertexIndex]);
		NewBox += ProjectedVertex;
	}

	return NewBox;
}

template<typename T>
TBox<T> TBox<T>::TransformProjectBy(const TMatrix<T>& ProjM) const
{
	// if we are not valid, return another invalid box.
	if (!IsValid)
	{
		return TBox<T>(ForceInit);
	}

	TVector<T> Vertices[8];
	GetVertices(Vertices);

	TBox<T> NewBox(ForceInit);

	for (int32 VertexIndex = 0; VertexIndex < UE_ARRAY_COUNT(Vertices); VertexIndex++)
	{
		TVector4<T> ProjectedVertex = ProjM.TransformPosition(Vertices[VertexIndex]);
		NewBox += ((TVector<T>)ProjectedVertex) / ProjectedVertex.W;
	}

	return NewBox;
}

template<typename T>
TBox<T> TBox<T>::Overlap(const TBox<T>& Other) const
{
	if (Intersect(Other) == false)
	{
		static TBox<T> EmptyBox(ForceInit);
		return EmptyBox;
	}

	// otherwise they overlap
	// so find overlapping box
	TVector<T> MinVector, MaxVector;

	MinVector.X = FMath::Max(Min.X, Other.Min.X);
	MaxVector.X = FMath::Min(Max.X, Other.Max.X);

	MinVector.Y = FMath::Max(Min.Y, Other.Min.Y);
	MaxVector.Y = FMath::Min(Max.Y, Other.Max.Y);

	MinVector.Z = FMath::Max(Min.Z, Other.Min.Z);
	MaxVector.Z = FMath::Min(Max.Z, Other.Max.Z);

	return TBox<T>(MinVector, MaxVector);
}

template<typename T>
uint32 GetTypeHash(const TBox<T>& Box)
{
	return !Box.IsValid ? 0 // All invalid boxes are the same
		: HashCombineFast(GetTypeHash(Box.Min), GetTypeHash(Box.Max));
}
} // namespace UE::Math

UE_DECLARE_LWC_TYPE(Box, 3);

//template<> struct TCanBulkSerialize<FBox3f> { enum { Value = true }; };
template<> struct TIsPODType<FBox3f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FBox3f> { enum { Value = true }; };

//template<> struct TCanBulkSerialize<FBox3d> { enum { Value = false }; };	// LWC_TODO: This can be done (via versioning) once LWC is fixed to on.
template<> struct TIsPODType<FBox3d> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FBox3d> { enum { Value = true }; };

/* FMath inline functions
 *****************************************************************************/

template<typename FReal>
inline bool FMath::PointBoxIntersection
	(
	const UE::Math::TVector<FReal>&		Point,
	const UE::Math::TBox<FReal>&		Box
	)
{
	return (Point.X >= Box.Min.X && Point.X <= Box.Max.X &&
			Point.Y >= Box.Min.Y && Point.Y <= Box.Max.Y &&
			Point.Z >= Box.Min.Z && Point.Z <= Box.Max.Z);
}

template<typename FReal>
inline bool FMath::LineBoxIntersection
	(
	const UE::Math::TBox<FReal>&	Box,
	const UE::Math::TVector<FReal>&	Start,
	const UE::Math::TVector<FReal>&	End,
	const UE::Math::TVector<FReal>&	StartToEnd
	)
{
	return LineBoxIntersection(Box, Start, End, StartToEnd, StartToEnd.Reciprocal());
}

template<typename FReal>
inline bool FMath::LineBoxIntersection
	(
	const UE::Math::TBox<FReal>&	Box,
	const UE::Math::TVector<FReal>&	Start,
	const UE::Math::TVector<FReal>&	End,
	const UE::Math::TVector<FReal>&	StartToEnd,
	const UE::Math::TVector<FReal>&	OneOverStartToEnd
	)
{
	UE::Math::TVector<FReal>	Time;
	bool	bStartIsOutside = false;

	if(Start.X < Box.Min.X)
	{
		bStartIsOutside = true;
		if(End.X >= Box.Min.X)
		{
			Time.X = (Box.Min.X - Start.X) * OneOverStartToEnd.X;
		}
		else
		{
			return false;
		}
	}
	else if(Start.X > Box.Max.X)
	{
		bStartIsOutside = true;
		if(End.X <= Box.Max.X)
		{
			Time.X = (Box.Max.X - Start.X) * OneOverStartToEnd.X;
		}
		else
		{
			return false;
		}
	}
	else
	{
		Time.X = 0.0f;
	}

	if(Start.Y < Box.Min.Y)
	{
		bStartIsOutside = true;
		if(End.Y >= Box.Min.Y)
		{
			Time.Y = (Box.Min.Y - Start.Y) * OneOverStartToEnd.Y;
		}
		else
		{
			return false;
		}
	}
	else if(Start.Y > Box.Max.Y)
	{
		bStartIsOutside = true;
		if(End.Y <= Box.Max.Y)
		{
			Time.Y = (Box.Max.Y - Start.Y) * OneOverStartToEnd.Y;
		}
		else
		{
			return false;
		}
	}
	else
	{
		Time.Y = 0.0f;
	}

	if(Start.Z < Box.Min.Z)
	{
		bStartIsOutside = true;
		if(End.Z >= Box.Min.Z)
		{
			Time.Z = (Box.Min.Z - Start.Z) * OneOverStartToEnd.Z;
		}
		else
		{
			return false;
		}
	}
	else if(Start.Z > Box.Max.Z)
	{
		bStartIsOutside = true;
		if(End.Z <= Box.Max.Z)
		{
			Time.Z = (Box.Max.Z - Start.Z) * OneOverStartToEnd.Z;
		}
		else
		{
			return false;
		}
	}
	else
	{
		Time.Z = 0.0f;
	}

	if(bStartIsOutside)
	{
		const FReal MaxTime = Max3(Time.X,Time.Y,Time.Z);

		if(MaxTime >= 0.0f && MaxTime <= 1.0f)
		{
			const UE::Math::TVector<FReal> Hit = Start + StartToEnd * MaxTime;
			const FReal BOX_SIDE_THRESHOLD = 0.1f;
			if(	Hit.X > Box.Min.X - BOX_SIDE_THRESHOLD && Hit.X < Box.Max.X + BOX_SIDE_THRESHOLD &&
				Hit.Y > Box.Min.Y - BOX_SIDE_THRESHOLD && Hit.Y < Box.Max.Y + BOX_SIDE_THRESHOLD &&
				Hit.Z > Box.Min.Z - BOX_SIDE_THRESHOLD && Hit.Z < Box.Max.Z + BOX_SIDE_THRESHOLD)
			{
				return true;
			}
		}

		return false;
	}
	else
	{
		return true;
	}
}

/**
 * Performs a sphere vs box intersection test using Arvo's algorithm:
 *
 *	for each i in (x, y, z)
 *		if (SphereCenter(i) < BoxMin(i)) d2 += (SphereCenter(i) - BoxMin(i)) ^ 2
 *		else if (SphereCenter(i) > BoxMax(i)) d2 += (SphereCenter(i) - BoxMax(i)) ^ 2
 *
 * @param SphereCenter the center of the sphere being tested against the AABB
 * @param RadiusSquared the size of the sphere being tested
 * @param AABB the box being tested against
 *
 * @return Whether the sphere/box intersect or not.
 */
template<typename FReal>
inline bool FMath::SphereAABBIntersection(const UE::Math::TVector<FReal>& SphereCenter, const FReal RadiusSquared, const UE::Math::TBox<FReal>& AABB)
{
	// Accumulates the distance as we iterate axis
	FReal DistSquared = 0.f;
	// Check each axis for min/max and add the distance accordingly
	// NOTE: Loop manually unrolled for > 2x speed up
	if (SphereCenter.X < AABB.Min.X)
	{
		DistSquared += FMath::Square(SphereCenter.X - AABB.Min.X);
	}
	else if (SphereCenter.X > AABB.Max.X)
	{
		DistSquared += FMath::Square(SphereCenter.X - AABB.Max.X);
	}
	if (SphereCenter.Y < AABB.Min.Y)
	{
		DistSquared += FMath::Square(SphereCenter.Y - AABB.Min.Y);
	}
	else if (SphereCenter.Y > AABB.Max.Y)
	{
		DistSquared += FMath::Square(SphereCenter.Y - AABB.Max.Y);
	}
	if (SphereCenter.Z < AABB.Min.Z)
	{
		DistSquared += FMath::Square(SphereCenter.Z - AABB.Min.Z);
	}
	else if (SphereCenter.Z > AABB.Max.Z)
	{
		DistSquared += FMath::Square(SphereCenter.Z - AABB.Max.Z);
	}
	// If the distance is less than or equal to the radius, they intersect
	return DistSquared <= RadiusSquared;
}

/**
 * Converts a sphere into a point plus radius squared for the test above
 */
template<typename FReal>
inline bool FMath::SphereAABBIntersection(const UE::Math::TSphere<FReal>& Sphere, const UE::Math::TBox<FReal>& AABB)
{
	FReal RadiusSquared = FMath::Square(Sphere.W);
	// If the distance is less than or equal to the radius, they intersect
	return SphereAABBIntersection(Sphere.Center, RadiusSquared, AABB);
}

