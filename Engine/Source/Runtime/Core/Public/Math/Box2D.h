// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Math/MathFwd.h" // IWYU pragma: export
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h" 
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/LargeWorldCoordinates.h"
#include "Misc/LargeWorldCoordinatesSerializer.h"
#include "Serialization/Archive.h"
#include "Templates/IsUECoreType.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

template <typename T> struct TIsPODType;

/**
 * Implements a rectangular 2D Box.
 */
namespace UE {
namespace Math {

template<typename T>
struct TBox2
{
public:
	using FReal = T;
	
	/** Holds the box's minimum point. */
	TVector2<T> Min;

	/** Holds the box's maximum point. */
	TVector2<T> Max;

	/** Holds a flag indicating whether this box is valid. */
	bool bIsValid = false;

public:

	/** Default constructor (no initialization). */
	[[nodiscard]] TBox2() = default;

	/**
	 * Creates and initializes a new box.
	 *
	 * The box extents are initialized to zero and the box is marked as invalid.
	 *
	 * @param EForceInit Force Init Enum.
	 */
	[[nodiscard]] explicit TBox2<T>( EForceInit )
	{
		Init();
	}

	/**
	 * Creates and initializes a new box from the specified parameters.
	 *
	 * @param InMin The box's minimum point.
	 * @param InMax The box's maximum point.
	 */
	[[nodiscard]] TBox2<T>( const TVector2<T>& InMin, const TVector2<T>& InMax )
		: Min(InMin)
		, Max(InMax)
		, bIsValid(true)
	{ }

	/**
	 * Creates and initializes a new box from the given set of points.
	 *
	 * @param Points Array of Points to create for the bounding volume.
	 * @param Count The number of points.
	 */
	[[nodiscard]] TBox2<T>( const TVector2<T>* Points, const int32 Count );

	/**
	 * Creates and initializes a new box from an array of points.
	 *
	 * @param Points Array of Points to create for the bounding volume.
	 */
	[[nodiscard]] TBox2<T>( const TArray<TVector2<T>>& Points );

public:

	/**
	 * Compares two boxes for equality.
	 *
	 * @param Other The other box to compare with.
	 * @return true if the boxes are equal, false otherwise.
	 */
	[[nodiscard]] bool operator==( const TBox2<T>& Other ) const
	{
		return (Min == Other.Min) && (Max == Other.Max);
	}

	/**
	 * Compares two boxes for inequality.
	 *
	 * @param Other The other box to compare with.
	 * @return true if the boxes are not equal, false otherwise.
	 */
	[[nodiscard]] bool operator!=( const TBox2<T>& Other ) const
	{
		return !(*this == Other);
	}

	/**
	 * Checks for equality with error-tolerant comparison.
	 *
	 * @param Other The box to compare.
	 * @param Tolerance Error tolerance.
	 * @return true if the boxes are equal within specified tolerance, otherwise false.
	 */
	[[nodiscard]] bool Equals( const TBox2<T>& Other, T Tolerance = UE_KINDA_SMALL_NUMBER ) const
	{
		return Min.Equals(Other.Min, Tolerance) && Max.Equals(Other.Max, Tolerance);
	}

	/**
	 * Adds to this bounding box to include a given point.
	 *
	 * @param Other The point to increase the bounding volume to.
	 * @return Reference to this bounding box after resizing to include the other point.
	 */
	inline TBox2<T>& operator+=( const TVector2<T> &Other );

	/**
	 * Gets the result of addition to this bounding volume.
	 *
	 * @param Other The other point to add to this.
	 * @return A new bounding volume.
	 */
	[[nodiscard]] TBox2<T> operator+( const TVector2<T>& Other ) const
	{
		return TBox2<T>(*this) += Other;
	}

	/**
	 * Adds to this bounding box to include a new bounding volume.
	 *
	 * @param Other The bounding volume to increase the bounding volume to.
	 * @return Reference to this bounding volume after resizing to include the other bounding volume.
	 */
	inline TBox2<T>& operator+=( const TBox2<T>& Other );

	/**
	 * Gets the result of addition to this bounding volume.
	 *
	 * @param Other The other volume to add to this.
	 * @return A new bounding volume.
	 */
	[[nodiscard]] TBox2<T> operator+( const TBox2<T>& Other ) const
	{
		return TBox2<T>(*this) += Other;
	}

	/**
	 * Gets reference to the min or max of this bounding volume.
	 *
	 * @param Index The index into points of the bounding volume.
	 * @return A reference to a point of the bounding volume.
	 */
   [[nodiscard]] TVector2<T>& operator[]( int32 Index )
	{
		check((Index >= 0) && (Index < 2));

		if (Index == 0)
		{
			return Min;
		}

		return Max;
	}

	/**
	 * Gets reference to the min or max of this bounding volume.
	 *
	 * @param Index The index into points of the bounding volume.
	 * @return A const reference to a point of the bounding volume.
	 */
   [[nodiscard]] const TVector2<T>& operator[]( int32 Index ) const
	{
		check((Index >= 0) && (Index < 2));

		if (Index == 0)
		{
			return Min;
		}

		return Max;
	}


public:

	/** 
	 * Calculates the distance of a point to this box.
	 *
	 * @param Point The point.
	 * @return The distance.
	 */
	[[nodiscard]] inline T ComputeSquaredDistanceToPoint( const TVector2<T>& Point ) const
	{
		// Accumulates the distance as we iterate axis
		T DistSquared = 0.f;
		
		if (Point.X < Min.X)
		{
			DistSquared += FMath::Square(Point.X - Min.X);
		}
		else if (Point.X > Max.X)
		{
			DistSquared += FMath::Square(Point.X - Max.X);
		}
		
		if (Point.Y < Min.Y)
		{
			DistSquared += FMath::Square(Point.Y - Min.Y);
		}
		else if (Point.Y > Max.Y)
		{
			DistSquared += FMath::Square(Point.Y - Max.Y);
		}
		
		return (T)DistSquared;
	}

	/** 
	 * Increase the bounding box volume.
	 *
	 * @param W The size to increase volume by.
	 * @return A new bounding box increased in size.
	 */
	[[nodiscard]] TBox2<T> ExpandBy( const T W ) const
	{
		return TBox2<T>(Min - TVector2<T>(W, W), Max + TVector2<T>(W, W));
	}

	/**
     * Returns a box of increased size.
     *
     * @param V The size to increase the volume by.
     * @return A new bounding box.
     */
	[[nodiscard]] TBox2<T> ExpandBy(const TVector2<T>& V) const
	{
		return TBox2<T>(Min - V, Max + V);
	}

	/**
	 * Gets the box area.
	 *
	 * @return Box area.
	 * @see GetCenter, GetCenterAndExtents, GetExtent, GetSize
	 */
	[[nodiscard]] T GetArea() const
	{
		return (Max.X - Min.X) * (Max.Y - Min.Y);
	}

	/**
	 * Gets the box's center point.
	 *
	 * @return Th center point.
	 * @see GetArea, GetCenterAndExtents, GetExtent, GetSize
	 */
	[[nodiscard]] TVector2<T> GetCenter() const
	{
		return TVector2<T>((Min + Max) * 0.5f);
	}

	/**
	 * Get the center and extents
	 *
	 * @param center[out] reference to center point
	 * @param Extents[out] reference to the extent around the center
	 * @see GetArea, GetCenter, GetExtent, GetSize
	 */
	void GetCenterAndExtents( TVector2<T> & center, TVector2<T> & Extents ) const
	{
		Extents = GetExtent();
		center = Min + Extents;
	}

	/**
	 * Calculates the closest point on or inside the box to a given point in space.
	 *
	 * @param Point The point in space.
	 *
	 * @return The closest point on or inside the box.
	 */
	[[nodiscard]] inline TVector2<T> GetClosestPointTo( const TVector2<T>& Point ) const;

	/**
	 * Gets the box extents around the center.
	 *
	 * @return Box extents.
	 * @see GetArea, GetCenter, GetCenterAndExtents, GetSize
	 */
	[[nodiscard]] TVector2<T> GetExtent() const
	{
		return 0.5f * TVector2<T>(Max - Min);
	}


	/**
	 * Gets the box size.
	 *
	 * @return Box size.
	 * @see GetArea, GetCenter, GetCenterAndExtents, GetExtent
	 */
	[[nodiscard]] TVector2<T> GetSize() const
	{
		return TVector2<T>(Max - Min);
	}

	/**
	 * Set the initial values of the bounding box to Zero.
	 */
	void Init()
	{
		// Zero padding for deterministic cooking with UPS serialize-as-zero for immutable/atomic noexport types
		FMemory::Memset(this, 0, sizeof(TBox2));
	}

	/**
	 * Returns the overlap box of two boxes
	 *
	 * @param Other The bounding box to test overlap
	 * @return the overlap box. Result will be invalid if they don't overlap
	 */
	[[nodiscard]] TBox2<T> Overlap( const TBox2<T>& Other ) const;

	/**
	 * Checks whether the given box intersects this box.
	 *
	 * @param Other bounding box to test intersection
	 * @return true if boxes intersect, false otherwise.
	 */
	[[nodiscard]] inline bool Intersect( const TBox2<T> & Other ) const;
	
	/**
	 * Checks whether the given point is inside this box.
	 *
	 * @param Point The point to test.
	 * @return true if the point is inside this box, otherwise false.
	 */
	[[nodiscard]] bool IsInside( const TVector2<T> & TestPoint ) const
	{
		return ((TestPoint.X > Min.X) && (TestPoint.X < Max.X) && (TestPoint.Y > Min.Y) && (TestPoint.Y < Max.Y));
	}

	/**
	 * Checks whether the given point is inside or on this box.
	 *
	 * @param Point The point to test.
	 * @return true if point is inside or on this box, otherwise false.
	 * @see IsInside
	 */
	[[nodiscard]] bool IsInsideOrOn( const TVector2<T>& TestPoint ) const
	{
		return ((TestPoint.X >= Min.X) && (TestPoint.X <= Max.X) && (TestPoint.Y >= Min.Y) && (TestPoint.Y <= Max.Y));
	}

	/** 
	 * Checks whether the given box is fully encapsulated by this box.
	 * 
	 * @param Other The box to test for encapsulation within the bounding volume.
	 * @return true if box is inside this volume, false otherwise.
	 */
	[[nodiscard]] bool IsInside( const TBox2<T>& Other ) const
	{
		return (IsInside(Other.Min) && IsInside(Other.Max));
	}

	/** 
	 * Shift bounding box position.
	 *
	 * @param The offset vector to shift by.
	 * @return A new shifted bounding box.
	 */
	[[nodiscard]] TBox2<T> ShiftBy( const TVector2<T>& Offset ) const
	{
		return TBox2<T>(Min + Offset, Max + Offset);
	}

	/**
	 * Returns a box with its center moved to the new destination.
	 *
	 * @param Destination The destination point to move center of box to.
	 * @return A new bounding box.
	 */
	[[nodiscard]] TBox2<T> MoveTo(const TVector2<T>& Destination) const
	{
		const TVector2<T> Offset = Destination - GetCenter();
		return TBox2<T>(Min + Offset, Max + Offset);
	}

	/**
	 * Get a textual representation of this box.
	 *
	 * @return A string describing the box.
	 */
	[[nodiscard]] FString ToString() const;

	/** 
     * Utility function to build an AABB from Origin and Extent 
     *
     * @param Origin The location of the bounding box.
     * @param Extent Half size of the bounding box.
     * @return A new axis-aligned bounding box.
     */
    [[nodiscard]] static TBox2<T> BuildAABB( const TVector2<T>& Origin, const TVector2<T>& Extent )
    {
    	TBox2<T> NewBox(Origin - Extent, Origin + Extent);

    	return NewBox;
    }

public:

	/**
	 * Serializes the bounding box.
	 *
	 * @param Ar The archive to serialize into.
	 * @param Box The box to serialize.
	 *
	 * @return Reference to the Archive after serialization.
	 */
	friend FArchive& operator<<( FArchive& Ar, TBox2<T>& Box )
	{
		return Ar << Box.Min << Box.Max << Box.bIsValid;
	}

	// Note: TBox2 is usually written via binary serialization. This function exists for SerializeFromMismatchedTag conversion usage. 
	bool Serialize(FArchive& Ar)
	{
		Ar << Min << Max;
		// Can't do Ar << bIsValid as that performs legacy UBOOL (uint32) serialization.
		uint8 bValid = bIsValid;
		Ar.Serialize(&bValid, sizeof(uint8));
		bIsValid = !!bValid;
		return true;
	}

	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		if constexpr (std::is_same_v<T, float>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Box2D, Box2f, Box2d);
		}
		else if constexpr (std::is_same_v<T, double>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Box2D, Box2d, Box2f);
		}
		else
		{
			static_assert(sizeof(T) == 0, "Unimplemented");
			return false;
		}
	}

	// Conversion from other type. 
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TBox2(const TBox2<FArg>& From)
		: Min((TVector2<T>)From.Min)
		, Max((TVector2<T>)From.Max)
		, bIsValid(From.bIsValid)
	{
	}
};


/* TBox2 inline functions
 *****************************************************************************/

template<typename T>
TBox2<T>::TBox2(const TVector2<T>* Points, const int32 Count)
	: Min(0.f, 0.f)
	, Max(0.f, 0.f)
	, bIsValid(false)
{
	for (int32 PointItr = 0; PointItr < Count; PointItr++)
	{
		*this += Points[PointItr];
	}
}

template<typename T>
TBox2<T>::TBox2(const TArray<TVector2<T>>& Points)
	: Min(0.f, 0.f)
	, Max(0.f, 0.f)
	, bIsValid(false)
{
	for(const TVector2<T>& EachPoint : Points)
	{
		*this += EachPoint;
	}
}


template<typename T>
inline TBox2<T>& TBox2<T>::operator+=( const TVector2<T> &Other )
{
	if (bIsValid)
	{
		Min.X = FMath::Min(Min.X, Other.X);
		Min.Y = FMath::Min(Min.Y, Other.Y);
	
		Max.X = FMath::Max(Max.X, Other.X);
		Max.Y = FMath::Max(Max.Y, Other.Y);
		
	}
	else
	{
		Min = Max = Other;
		bIsValid = true;
	}

	return *this;
}

template<typename T>
inline TBox2<T>& TBox2<T>::operator+=( const TBox2<T>& Other )
{
	if (bIsValid && Other.bIsValid)
	{
		Min.X = FMath::Min(Min.X, Other.Min.X);
		Min.Y = FMath::Min(Min.Y, Other.Min.Y);

		Max.X = FMath::Max(Max.X, Other.Max.X);
		Max.Y = FMath::Max(Max.Y, Other.Max.Y);
	}
	else if (Other.bIsValid)
	{
		*this = Other;
	}

	return *this;
}

template<typename T>
inline TVector2<T> TBox2<T>::GetClosestPointTo( const TVector2<T>& Point ) const
{
	// start by considering the point inside the box
	TVector2<T> ClosestPoint = Point;

	// now clamp to inside box if it's outside
	if (Point.X < Min.X)
	{
		ClosestPoint.X = Min.X;
	}
	else if (Point.X > Max.X)
	{
		ClosestPoint.X = Max.X;
	}

	// now clamp to inside box if it's outside
	if (Point.Y < Min.Y)
	{
		ClosestPoint.Y = Min.Y;
	}
	else if (Point.Y > Max.Y)
	{
		ClosestPoint.Y = Max.Y;
	}

	return ClosestPoint;
}

template<typename T>
TBox2<T> TBox2<T>::Overlap(const TBox2<T>& Other) const
{
	if (Intersect(Other) == false)
	{
		static TBox2<T> EmptyBox(ForceInit);
		return EmptyBox;
	}

	// otherwise they overlap
	// so find overlapping box
	TVector2<T> MinVector, MaxVector;

	MinVector.X = FMath::Max(Min.X, Other.Min.X);
	MaxVector.X = FMath::Min(Max.X, Other.Max.X);

	MinVector.Y = FMath::Max(Min.Y, Other.Min.Y);
	MaxVector.Y = FMath::Min(Max.Y, Other.Max.Y);

	return TBox2<T>(MinVector, MaxVector);
}

template<typename T>
inline bool TBox2<T>::Intersect( const TBox2<T> & Other ) const
{
	if ((Min.X > Other.Max.X) || (Other.Min.X > Max.X))
	{
		return false;
	}

	if ((Min.Y > Other.Max.Y) || (Other.Min.Y > Max.Y))
	{
		return false;
	}

	return true;
}

template<typename T>
UE_FORCEINLINE_HINT FString TBox2<T>::ToString() const
{
	return FString::Printf(TEXT("bIsValid=%s, Min=(%s), Max=(%s)"), bIsValid ? TEXT("true") : TEXT("false"), *Min.ToString(), *Max.ToString());
}

} // namespace Math
} // namespace UE

UE_DECLARE_LWC_TYPE(Box2,, FBox2D);

//template<> struct TCanBulkSerialize<FBox2f> { enum { Value = true }; };
template<> struct TIsPODType<FBox2f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FBox2f> { enum { Value = true }; };

//template<> struct TCanBulkSerialize<FBox2d> { enum { Value = false }; };	// LWC_TODO: This can be done (via versioning) once LWC is fixed to on.
template<> struct TIsPODType<FBox2d> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FBox2d> { enum { Value = true }; };

/* FMath inline functions
 *****************************************************************************/

template<typename FReal>
inline bool FMath::LineBoxIntersection2D
	(
	const UE::Math::TBox2<FReal>&		Box,
	const UE::Math::TVector2<FReal>&	Start,
	const UE::Math::TVector2<FReal>&	End
	)
{
	using VectorType = UE::Math::TVector2<FReal>;
	
	// The segment is intersecting if its endpoints lie on any of the edges or inside the box
	if (Box.IsInsideOrOn(Start) || Box.IsInsideOrOn(End))
	{
		return true;
	}
	
	const VectorType BoxSize = Box.GetSize();
	const FVector SegmentStart(Start.X, Start.Y, 0);
	const FVector SegmentEnd(End.X, End.Y, 0);
	FVector OutIntersectionPoint; // We don't do anything with this value, but it is needed by the SegmentIntersection2D function

	//  A ____ B
	//  |      |
	//  |      |
	//  C ____ D
	const FVector PointA = FVector(Box.Min.X, Box.Min.Y, 0);
	const FVector PointB = PointA + FVector(BoxSize.X, 0, 0);
	
	if (SegmentIntersection2D(SegmentStart, SegmentEnd, PointA, PointB, OutIntersectionPoint))
	{
		return true;
	}

	const FVector PointC = PointA + FVector(0, BoxSize.Y, 0);
	if (SegmentIntersection2D(SegmentStart, SegmentEnd, PointA, PointC, OutIntersectionPoint))
	{
		return true;
	}
	
	const FVector PointD = PointA + FVector(BoxSize.X, BoxSize.Y, 0);;
	if (SegmentIntersection2D(SegmentStart, SegmentEnd, PointB, PointD, OutIntersectionPoint))
	{
		return true;
	}
	
	if (SegmentIntersection2D(SegmentStart, SegmentEnd, PointC, PointD, OutIntersectionPoint))
	{
		return true;
	}
	
	return false;
}