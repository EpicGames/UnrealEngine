// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/MathFwd.h" // IWYU pragma: export
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "Math/Vector.h"
#include "Math/Sphere.h"
#include "Math/Box.h"
#include "Misc/LargeWorldCoordinatesSerializer.h"

/**
 * Structure for a combined axis aligned bounding box and bounding sphere with the same origin. (28 bytes).
 */
namespace UE
{
namespace Math
{

template<typename T, typename TExtent>
struct TBoxSphereBounds
{
	using FReal = T;

	/** Holds the origin of the bounding box and sphere. */
	TVector<T>	Origin;

	/** Holds the extent of the bounding box. */
	TVector<TExtent> BoxExtent;

	/** Holds the radius of the bounding sphere. */
	TExtent SphereRadius;

public:

	/** Default constructor. */
	[[nodiscard]] TBoxSphereBounds() = default;

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param EForceInit Force Init Enum.
	 */
	[[nodiscard]] explicit inline TBoxSphereBounds( EForceInit )
		: Origin(ForceInit)
		, BoxExtent(ForceInit)
		, SphereRadius(0)
	{
		DiagnosticCheckNaN();
	}

	/**
	 * Creates and initializes a new instance from the specified parameters.
	 *
	 * @param InOrigin origin of the bounding box and sphere.
	 * @param InBoxExtent half size of box.
	 * @param InSphereRadius radius of the sphere.
	 */
	[[nodiscard]] TBoxSphereBounds( const TVector<T>& InOrigin, const TVector<TExtent>& InBoxExtent, TExtent InSphereRadius )
		: Origin(InOrigin)
		, BoxExtent(InBoxExtent)
		, SphereRadius(InSphereRadius)
	{
		DiagnosticCheckNaN();
	}

	/**
	 * Creates and initializes a new instance from the given Box and Sphere.
	 *
	 * @param Box The bounding box.
	 * @param Sphere The bounding sphere.
	 */
	[[nodiscard]] TBoxSphereBounds( const TBox<T>& Box, const TSphere<T>& Sphere )
	{
		TVector<T> LocalExtent;
		Box.GetCenterAndExtents(Origin, LocalExtent);
		BoxExtent = TVector<TExtent>(LocalExtent);

		SphereRadius = (TExtent)FMath::Min(LocalExtent.Size(), (Sphere.Center - Origin).Size() + Sphere.W);

		DiagnosticCheckNaN();
	}
	
	/**
	 * Creates and initializes a new instance the given Box.
	 *
	 * The sphere radius is taken from the extent of the box.
	 *
	 * @param Box The bounding box.
	 */
	[[nodiscard]] TBoxSphereBounds( const TBox<T>& Box )
	{
		TVector<T> LocalExtent;
		Box.GetCenterAndExtents(Origin, LocalExtent);
		BoxExtent = TVector<TExtent>(LocalExtent);

		SphereRadius = BoxExtent.Size();

		DiagnosticCheckNaN();
	}

	/**
	 * Creates and initializes a new instance for the given sphere.
	 */
	[[nodiscard]] TBoxSphereBounds( const TSphere<T>& Sphere )
	{
		Origin = Sphere.Center;
		SphereRadius = Sphere.W;
		BoxExtent = TVector<TExtent>(SphereRadius);

		DiagnosticCheckNaN();
	}

	/**
	 * Creates and initializes a new instance from the given set of points.
	 *
	 * The sphere radius is taken from the extent of the box.
	 *
	 * @param Points The points to be considered for the bounding box.
	 * @param NumPoints Number of points in the Points array.
	 */
	[[nodiscard]] TBoxSphereBounds( const TVector<T>* Points, uint32 NumPoints );

	// Conversion to other type.
	template<typename TFrom, typename TExtentFrom UE_REQUIRES(!(std::is_same_v<T, TFrom> && std::is_same_v<TExtent, TExtentFrom>))>
	explicit TBoxSphereBounds(const TBoxSphereBounds<TFrom, TExtentFrom>& From)
		: TBoxSphereBounds<T, TExtent>(TVector<T>(From.Origin), TVector<TExtent>(From.BoxExtent), (TExtent)From.SphereRadius)
	{
	}

public:

	bool Serialize(FArchive &Ar);
	bool SerializeFromMismatchedTag(FName StructTag, FArchive &Ar)
	{
		if constexpr (std::is_same_v<T, float>)
		{
			// Falls back to UseSerializeItem to convert per property.
			return false;
		}
		else if constexpr (std::is_same_v<T, double>)
		{
			// Falls back to UseSerializeItem to convert per property.
			return false;
		}
		else
		{
			static_assert(sizeof(T) == 0, "Unimplemented");
			return false;
		}
	}

	/**
	 * Constructs a bounding volume containing both this and B.
	 *
	 * @param Other The other bounding volume.
	 * @return The combined bounding volume.
	 */
	[[nodiscard]] inline TBoxSphereBounds<T, TExtent> operator+( const TBoxSphereBounds<T, TExtent>& Other ) const;

	/**
	 * Compare bounding volume this and Other.
	 *
	 * @param Other The other bounding volume.
	 * @return true of they match.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TBoxSphereBounds<T, TExtent>& Other) const;
	
	/**
	 * Compare bounding volume this and Other.
	 *
	 * @param Other The other bounding volume.
	 * @return true of they do not match.
	 */	
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TBoxSphereBounds<T, TExtent>& Other) const;

public:

	/**
	 * Calculates the squared distance from a point to a bounding box
	 *
	 * @param Point The point.
	 * @return The distance.
	 */
	[[nodiscard]] inline T ComputeSquaredDistanceFromBoxToPoint( const TVector<T>& Point ) const
	{
		TVector<T> Mins = Origin - BoxExtent;
		TVector<T> Maxs = Origin + BoxExtent;

		return ::ComputeSquaredDistanceFromBoxToPoint(Mins, Maxs, Point);
	}

	/**
	 * Test whether the spheres from two BoxSphereBounds intersect/overlap.
	 * 
	 * @param  A First BoxSphereBounds to test.
	 * @param  B Second BoxSphereBounds to test.
	 * @param  Tolerance Error tolerance added to test distance.
	 * @return true if spheres intersect, false otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT static bool SpheresIntersect(const TBoxSphereBounds<T, TExtent>& A, const TBoxSphereBounds<T, TExtent>& B, TExtent Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return (A.Origin - B.Origin).SizeSquared() <= FMath::Square(FMath::Max<TExtent>(0, A.SphereRadius + B.SphereRadius + Tolerance));
	}

	/**
	 * Test whether the boxes from two BoxSphereBounds intersect/overlap.
	 * 
	 * @param  A First BoxSphereBounds to test.
	 * @param  B Second BoxSphereBounds to test.
	 * @return true if boxes intersect, false otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT static bool BoxesIntersect(const TBoxSphereBounds<T, TExtent>& A, const TBoxSphereBounds<T, TExtent>& B)
	{
		return A.GetBox().Intersect(B.GetBox());
	}

	/**
	 * Gets the bounding box.
	 *
	 * @return The bounding box.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TBox<T> GetBox() const
	{
		return TBox<T>(Origin - BoxExtent,Origin + BoxExtent);
	}

	/**
	 * Gets the extrema for the bounding box.
	 *
	 * @param Extrema 1 for positive extrema from the origin, else negative
	 * @return The boxes extrema
	 */
	[[nodiscard]] TVector<T> GetBoxExtrema( uint32 Extrema ) const
	{
		if (Extrema)
		{
			return Origin + BoxExtent;
		}

		return Origin - BoxExtent;
	}

	/**
	 * Gets the bounding sphere.
	 *
	 * @return The bounding sphere.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TSphere<T> GetSphere() const
	{
		return TSphere<T>(Origin,SphereRadius);
	}

	/**
	 * Increase the size of the box and sphere by a given size.
	 *
	 * @param ExpandAmount The size to increase by.
	 * @return A new box with the expanded size.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TBoxSphereBounds<T, TExtent> ExpandBy(TExtent ExpandAmount ) const
	{
		return TBoxSphereBounds(Origin, BoxExtent + ExpandAmount, SphereRadius + ExpandAmount);
	}

	/**
	 * Gets a bounding volume transformed by a matrix.
	 *
	 * @param M The matrix.
	 * @return The transformed volume.
	 */
	[[nodiscard]] TBoxSphereBounds<T, TExtent> TransformBy( const TMatrix<T>& M ) const;

	/**
	 * Gets a bounding volume transformed by a FTransform object.
	 *
	 * @param M The FTransform object.
	 * @return The transformed volume.
	 */
	[[nodiscard]] TBoxSphereBounds<T, TExtent> TransformBy( const TTransform<T>& M ) const;

	/**
	 * Get a textual representation of this bounding box.
	 *
	 * @return Text describing the bounding box.
	 */
	[[nodiscard]] FString ToString() const;

	/**
	 * Constructs a bounding volume containing both A and B.
	 *
	 * This is a legacy version of the function used to compute primitive bounds, to avoid the need to rebuild lighting after the change.
	 */
	[[nodiscard]] friend TBoxSphereBounds<T, TExtent> Union( const TBoxSphereBounds<T, TExtent>& A,const TBoxSphereBounds<T, TExtent>& B )
	{
		return A + B;
	}

#if ENABLE_NAN_DIAGNOSTIC
	inline void DiagnosticCheckNaN() const
	{
		if (Origin.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("Origin contains NaN: %s"), *Origin.ToString());
		}
		if (BoxExtent.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("BoxExtent contains NaN: %s"), *BoxExtent.ToString());
		}
		if (FMath::IsNaN(SphereRadius) || !FMath::IsFinite(SphereRadius))
		{
			logOrEnsureNanError(TEXT("SphereRadius contains NaN: %f"), SphereRadius);
		}
	}
#else
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN() const {}
#endif

	[[nodiscard]] inline bool ContainsNaN() const
	{
		return Origin.ContainsNaN() || BoxExtent.ContainsNaN() || !FMath::IsFinite(SphereRadius);
	}


public:
	/**
	 * Utility class to build a TBoxSphereBounds by adding other math primitives.
	 * Correctly handles the initial insertion (instead of potentially adding the ZeroVector by default)
	 */
	struct Builder
	{
		UE_FORCEINLINE_HINT Builder& operator+=(const TBoxSphereBounds<T, TExtent>& Other)
		{
			return Append(Other);
		}

		UE_FORCEINLINE_HINT Builder& operator+=(const TBox<T>& Box)
		{
			return Append(Box);
		}

		UE_FORCEINLINE_HINT Builder& operator+=(const TSphere<T>& Sphere)
		{
			return Append(Sphere);
		}

		UE_FORCEINLINE_HINT Builder& operator+=(const TVector<T>& Point)
		{
			return Append(TBoxSphereBounds<T, TExtent>(Point, TVector<T>::ZeroVector, T(0)));
		}

		UE_FORCEINLINE_HINT bool IsValid() const
		{
			return BoxSphereBounds.IsSet();
		}

		UE_FORCEINLINE_HINT operator TBoxSphereBounds<T, TExtent>() const
		{
			return BoxSphereBounds.Get(TBoxSphereBounds<T, TExtent>(ForceInitToZero));
		}

	private:
		inline Builder& Append(const TBoxSphereBounds<T, TExtent>& Other)
		{
			if (IsValid())
			{
				*BoxSphereBounds = *BoxSphereBounds + Other;
			}
			else
			{
				BoxSphereBounds.Emplace(Other);
			}
			return *this;
		}

		TOptional<TBoxSphereBounds<T, TExtent>> BoxSphereBounds;
	};
};


/* TBoxSphereBounds<T, TExtent> inline functions
 *****************************************************************************/

/**
 * Serializes the given bounding volume from or into the specified archive.
 *
 * @param Ar The archive to serialize from or into.
 * @param Bounds The bounding volume to serialize.
 * @return The archive..
 */
inline FArchive& operator<<(FArchive& Ar, TBoxSphereBounds<float, float>& Bounds)
{
	Ar << Bounds.Origin << Bounds.BoxExtent << Bounds.SphereRadius;
	return Ar;
}

/**
 * Serializes the given bounding volume from or into the specified archive.
 *
 * @param Ar The archive to serialize from or into.
 * @param Bounds The bounding volume to serialize.
 * @return The archive..
 */
inline FArchive& operator<<(FArchive& Ar, TBoxSphereBounds<double, double>& Bounds)
{
	Ar << Bounds.Origin << Bounds.BoxExtent;
	if (Ar.UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
	{
		Ar << Bounds.SphereRadius;
	}
	else
	{
		checkf(Ar.IsLoading(), TEXT("float -> double conversion applied outside of load!"));
		// Stored as floats, so serialize float and copy.
		float Radius;
		Ar << Radius;
		Bounds.SphereRadius = Radius;
	}

	return Ar;
}

template<typename T, typename TExtent>
inline TBoxSphereBounds<T, TExtent>::TBoxSphereBounds( const TVector<T>* Points, uint32 NumPoints )
{
	TBox<T> BoundingBox(ForceInit);

	// find an axis aligned bounding box for the points.
	for (uint32 PointIndex = 0; PointIndex < NumPoints; PointIndex++)
	{
		BoundingBox += Points[PointIndex];
	}

	TVector<T> LocalExtent;
	BoundingBox.GetCenterAndExtents(Origin, LocalExtent);
	BoxExtent = TVector<TExtent>(LocalExtent);

	// using the center of the bounding box as the origin of the sphere, find the radius of the bounding sphere.
	TExtent SquaredSphereRadius = 0;

	for (uint32 PointIndex = 0; PointIndex < NumPoints; PointIndex++)
	{
		SquaredSphereRadius = FMath::Max<TExtent>(SquaredSphereRadius, (Points[PointIndex] - Origin).SizeSquared());	// LWC_TODO: Precision loss
	}

	SphereRadius = FMath::Sqrt(SquaredSphereRadius);

	DiagnosticCheckNaN();
}

template<typename T, typename TExtent>
inline TBoxSphereBounds<T, TExtent> TBoxSphereBounds<T, TExtent>::operator+( const TBoxSphereBounds<T, TExtent>& Other ) const
{
	TBox<T> BoundingBox(ForceInit);

	BoundingBox += (this->Origin - this->BoxExtent);
	BoundingBox += (this->Origin + this->BoxExtent);
	BoundingBox += (Other.Origin - Other.BoxExtent);
	BoundingBox += (Other.Origin + Other.BoxExtent);

	// build a bounding sphere from the bounding box's origin and the radii of A and B.
	TBoxSphereBounds<T, TExtent> Result(BoundingBox);

	Result.SphereRadius = FMath::Min<TExtent>(Result.SphereRadius, FMath::Max<TExtent>((Origin - Result.Origin).Size() + SphereRadius, (Other.Origin - Result.Origin).Size() + Other.SphereRadius));
	Result.DiagnosticCheckNaN();

	return Result;
}

template<typename T, typename TExtent>
UE_FORCEINLINE_HINT bool TBoxSphereBounds<T, TExtent>::operator==(const TBoxSphereBounds<T, TExtent>& Other) const
{
	return Origin == Other.Origin && BoxExtent == Other.BoxExtent &&  SphereRadius == Other.SphereRadius;
}

template<typename T, typename TExtent>
UE_FORCEINLINE_HINT bool TBoxSphereBounds<T, TExtent>::operator!=(const TBoxSphereBounds<T, TExtent>& Other) const
{
	return !(*this == Other);
}

template<typename T, typename TExtent>
inline bool TBoxSphereBounds<T, TExtent>::Serialize(FArchive &Ar)
{
	Ar << *this;
	return true;
}

template<typename T, typename TExtent>
UE_FORCEINLINE_HINT FString TBoxSphereBounds<T, TExtent>::ToString() const
{
	return FString::Printf(TEXT("Origin=%s, BoxExtent=(%s), SphereRadius=(%f)"), *Origin.ToString(), *BoxExtent.ToString(), SphereRadius);
}

template<typename T, typename TExtent>
TBoxSphereBounds<T, TExtent> TBoxSphereBounds<T, TExtent>::TransformBy(const TMatrix<T>& M) const
{
#if ENABLE_NAN_DIAGNOSTIC
	if (M.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("Input Matrix contains NaN/Inf! %s"), *M.ToString());
	}
#endif

	TBoxSphereBounds<T> Result;

	const TVectorRegisterType<T> VecOrigin = VectorLoadFloat3(&Origin);
	const TVectorRegisterType<T> VecExtent = VectorLoadFloat3(&BoxExtent);

	const TVectorRegisterType<T> m0 = VectorLoadAligned(M.M[0]);
	const TVectorRegisterType<T> m1 = VectorLoadAligned(M.M[1]);
	const TVectorRegisterType<T> m2 = VectorLoadAligned(M.M[2]);
	const TVectorRegisterType<T> m3 = VectorLoadAligned(M.M[3]);

	TVectorRegisterType<T> NewOrigin = VectorMultiply(VectorReplicate(VecOrigin, 0), m0);
	NewOrigin = VectorMultiplyAdd(VectorReplicate(VecOrigin, 1), m1, NewOrigin);
	NewOrigin = VectorMultiplyAdd(VectorReplicate(VecOrigin, 2), m2, NewOrigin);
	NewOrigin = VectorAdd(NewOrigin, m3);

	TVectorRegisterType<T> NewExtent = VectorAbs(VectorMultiply(VectorReplicate(VecExtent, 0), m0));
	NewExtent = VectorAdd(NewExtent, VectorAbs(VectorMultiply(VectorReplicate(VecExtent, 1), m1)));
	NewExtent = VectorAdd(NewExtent, VectorAbs(VectorMultiply(VectorReplicate(VecExtent, 2), m2)));

	VectorStoreFloat3(NewExtent, &(Result.BoxExtent.X));
	VectorStoreFloat3(NewOrigin, &(Result.Origin.X));

	TVectorRegisterType<T> MaxRadius = VectorMultiply(m0, m0);
	MaxRadius = VectorMultiplyAdd(m1, m1, MaxRadius);
	MaxRadius = VectorMultiplyAdd(m2, m2, MaxRadius);
	MaxRadius = VectorMax(VectorMax(MaxRadius, VectorReplicate(MaxRadius, 1)), VectorReplicate(MaxRadius, 2));
	Result.SphereRadius = FMath::Sqrt(VectorGetComponent(MaxRadius, 0)) * SphereRadius;

	// For non-uniform scaling, computing sphere radius from a box results in a smaller sphere.
	T const BoxExtentMagnitude = FMath::Sqrt(VectorDot3Scalar(NewExtent, NewExtent));
	Result.SphereRadius = FMath::Min(Result.SphereRadius, BoxExtentMagnitude);

	Result.DiagnosticCheckNaN();
	return TBoxSphereBounds<T, TExtent>(Result);
}

/**
 * Gets a bounding volume transformed by a FTransform object.
 *
 * @param M The FTransform object.
 * @return The transformed volume.
 */
 template<typename T, typename TExtent>
TBoxSphereBounds<T, TExtent> TBoxSphereBounds<T, TExtent>::TransformBy(const TTransform<T>& M) const
{
#if ENABLE_NAN_DIAGNOSTIC
	M.DiagnosticCheckNaN_All();
#endif

	const TMatrix<T> Mat = M.ToMatrixWithScale();
	TBoxSphereBounds<T, TExtent> Result = TransformBy(Mat);
	return Result;
}

} // namespace UE::Math
} // namespace UE

template <> struct TIsPODType<FBoxSphereBounds3f> { enum { Value = true }; };
template <> struct TIsPODType<FBoxSphereBounds3d> { enum { Value = true }; };
template <> struct TIsUECoreVariant<FBoxSphereBounds3f> { enum { Value = true }; };
template <> struct TIsUECoreVariant<FBoxSphereBounds3d> { enum { Value = true }; };
template <> struct TIsUECoreVariant<FCompactBoxSphereBounds3d> { enum { Value = true }; };
