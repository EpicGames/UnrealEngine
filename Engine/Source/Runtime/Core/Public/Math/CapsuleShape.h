// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector.h"

/**
 * Structure for capsules.
 *
 * A capsule consists of two sphere connected by a cylinder.
 */
namespace UE
{
namespace Math
{

template<typename T>
struct TCapsuleShape
{
	using FReal = T;

	/** The capsule's center point. */
	TVector<T> Center;

	/** The capsule's radius. */
	T Radius;

	/** The capsule's orientation in space. */
	TVector<T> Orientation;

	/** The capsule's length. */
	T Length;

public:

	/** Default constructor. */
	[[nodiscard]] TCapsuleShape() { }

	/**
	 * Create and inintialize a new instance.
	 *
	 * @param InCenter The capsule's center point.
	 * @param InRadius The capsule's radius.
	 * @param InOrientation The capsule's orientation in space.
	 * @param InLength The capsule's length.
	 */
	[[nodiscard]] TCapsuleShape(TVector<T> InCenter, T InRadius, TVector<T> InOrientation, T InLength)
		: Center(InCenter)
		, Radius(InRadius)
		, Orientation(InOrientation)
		, Length(InLength)
	{ }

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	[[nodiscard]] explicit TCapsuleShape(const TCapsuleShape<FArg>& From)
		: TCapsuleShape<T>(TVector<T>(From.Center), (T)From.Radius, TVector<T>(From.Orientation), (T)From.Length)
	{
	}

	[[nodiscard]] bool IsInside(const TVector<T>& In, T Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		const TVector<T> HalfLengthAxis = 0.5 * Length * Orientation;
		const TVector<T> SegmentStart = Center - HalfLengthAxis;
		const TVector<T> SegmentEnd = Center + HalfLengthAxis;
		const TVector<T> Projection = FMath::ClosestPointOnSegment(In, SegmentStart, SegmentEnd);
		return (Projection - In).SizeSquared() <= FMath::Square(Radius + Tolerance);
	}
};

}	// namespace UE::Math
}	// namespace UE

UE_DECLARE_LWC_TYPE(CapsuleShape, 3);

template<> struct TIsUECoreVariant<FCapsuleShape3f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FCapsuleShape3d> { enum { Value = true }; };