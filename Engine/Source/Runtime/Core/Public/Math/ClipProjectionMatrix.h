// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Plane.h"
#include "Math/Matrix.h"

namespace UE {
namespace Math {

/**
 * Realigns the near plane for an existing projection matrix 
 * with an arbitrary clip plane
 * from: http://sourceforge.net/mailarchive/message.php?msg_id=000901c26324%242181ea90%24a1e93942%40firefly
 * Updated for the fact that our FPlane uses Ax+By+Cz=D.
 */
template<typename T>
struct TClipProjectionMatrix : public TMatrix<T>
{
public:
	using TMatrix<T>::M;
	/**
	 * Constructor
	 *
	 * @param	SrcProjMat - source projection matrix to premultiply with the clip matrix
	 * @param	Plane - clipping plane used to build the clip matrix (assumed to be in camera space)
	 */
	TClipProjectionMatrix( const TMatrix<T>& SrcProjMat, const TPlane<T>& Plane );

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TClipProjectionMatrix(const TClipProjectionMatrix<FArg>& From)
		: TMatrix<T>(From)
	{
	}
	
private:
	/** return sign of a number */
	inline T sgn(T a );
};


template<typename T>
inline TClipProjectionMatrix<T>::TClipProjectionMatrix( const TMatrix<T>& SrcProjMat, const TPlane<T>& Plane ) :
TMatrix<T>(SrcProjMat)
{
	// Calculate the clip-space corner point opposite the clipping plane
	// as (sgn(clipPlane.x), sgn(clipPlane.y), 1, 1) and
	// transform it into camera space by multiplying it
	// by the inverse of the projection matrix
	TPlane<T> CornerPlane( 
		sgn(Plane.X) / SrcProjMat.M[0][0],
		sgn(Plane.Y) / SrcProjMat.M[1][1],
		1.0f,
		-(1.0f - SrcProjMat.M[2][2]) / SrcProjMat.M[3][2]
	);

	// Calculate the scaled plane vector
	TPlane<T> ProjPlane( Plane * (1.0f / (Plane | CornerPlane)) );

	// use the projected space clip plane in z column 
	// Note: (account for our negated W coefficient)
	M[0][2] = ProjPlane.X;
	M[1][2] = ProjPlane.Y;
	M[2][2] = ProjPlane.Z;
	M[3][2] = -ProjPlane.W;
}

template<typename T>
inline T TClipProjectionMatrix<T>::sgn(T a)
{
	if (a > 0.0f) return (1.0f);
	if (a < 0.0f) return (-1.0f);
	return (0.0f);
}

} // namespace Math
} // namespace UE

UE_DECLARE_LWC_TYPE(ClipProjectionMatrix, 44);

template<> struct TIsUECoreVariant<FClipProjectionMatrix44f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FClipProjectionMatrix44d> { enum { Value = true }; };
