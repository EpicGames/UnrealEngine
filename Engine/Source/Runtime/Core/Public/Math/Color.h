// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PreprocessorHelpers.h"
#include "Math/MathFwd.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Crc.h"
#include "Misc/Parse.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryLayout.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveNameHelpers.h"
#include "Serialization/StructuredArchiveSlots.h"

class FFloat16Color;
class FMemoryImageWriter;
class FMemoryUnfreezeContent;
class FPointerTableBase;
class FSHA1;
struct FColor;
template <typename T> struct TIsPODType;

/**
 * Enum for the different kinds of gamma spaces we expect to need to convert from/to.
 */
enum class EGammaSpace : uint8
{
	/** No gamma correction is applied to this space, the incoming colors are assumed to already be in linear space. */
	Linear,
	/** A simplified sRGB gamma correction is applied, pow(1/2.2). */
	Pow22,
	/** Use the standard sRGB conversion. */
	sRGB,

	Invalid
};


/**
 * A linear, 32-bit/component floating point RGBA color.
 */
struct FLinearColor
{
	union
	{
		struct 
		{
			float	R,
					G,
					B,
					A;
		};

		UE_DEPRECATED(all, "For internal use only.")
		float RGBA[4];
	};

	/** Static lookup table used for FColor -> FLinearColor conversion. Pow(2.2) */
	static float Pow22OneOver255Table[256];

	/** Static lookup table used for fast FColor -> FLinearColor conversion. sRGB 
	 * 
	 *  Color > 0.04045 ? pow( Color * (1.0 / 1.055) + 0.0521327, 2.4 ) : Color * (1.0 / 12.92);
	 */
	static constexpr float sRGBToLinearTable[] =
	{
		0.0f,
		0.000303526983548838f, 0.000607053967097675f, 0.000910580950646512f, 0.00121410793419535f, 0.00151763491774419f,
		0.00182116190129302f, 0.00212468888484186f, 0.0024282158683907f, 0.00273174285193954f, 0.00303526983548838f,
		0.00334653564113713f, 0.00367650719436314f, 0.00402471688178252f, 0.00439144189356217f, 0.00477695332960869f,
		0.005181516543916f, 0.00560539145834456f, 0.00604883284946662f, 0.00651209061157708f, 0.00699540999852809f,
		0.00749903184667767f, 0.00802319278093555f, 0.0085681254056307f, 0.00913405848170623f, 0.00972121709156193f,
		0.0103298227927056f, 0.0109600937612386f, 0.0116122449260844f, 0.012286488094766f, 0.0129830320714536f,
		0.0137020827679224f, 0.0144438433080002f, 0.0152085141260192f, 0.0159962930597398f, 0.0168073754381669f,
		0.0176419541646397f, 0.0185002197955389f, 0.0193823606149269f, 0.0202885627054049f, 0.0212190100154473f,
		0.0221738844234532f, 0.02315336579873f, 0.0241576320596103f, 0.0251868592288862f, 0.0262412214867272f,
		0.0273208912212394f, 0.0284260390768075f, 0.0295568340003534f, 0.0307134432856324f, 0.0318960326156814f,
		0.0331047661035236f, 0.0343398063312275f, 0.0356013143874111f, 0.0368894499032755f, 0.0382043710872463f,
		0.0395462347582974f, 0.0409151963780232f, 0.0423114100815264f, 0.0437350287071788f, 0.0451862038253117f,
		0.0466650857658898f, 0.0481718236452158f, 0.049706565391714f, 0.0512694577708345f, 0.0528606464091205f,
		0.0544802758174765f, 0.0561284894136735f, 0.0578054295441256f, 0.0595112375049707f, 0.0612460535624849f,
		0.0630100169728596f, 0.0648032660013696f, 0.0666259379409563f, 0.0684781691302512f, 0.070360094971063f,
		0.0722718499453493f, 0.0742135676316953f, 0.0761853807213167f, 0.0781874210336082f, 0.0802198195312533f,
		0.0822827063349132f, 0.0843762107375113f, 0.0865004612181274f, 0.0886555854555171f, 0.0908417103412699f,
		0.0930589619926197f, 0.0953074657649191f, 0.0975873462637915f, 0.0998987273569704f, 0.102241732185838f,
		0.104616483176675f, 0.107023102051626f, 0.109461709839399f, 0.1119324268857f, 0.114435372863418f,
		0.116970666782559f, 0.119538426999953f, 0.122138771228724f, 0.124771816547542f, 0.127437679409664f,
		0.130136475651761f, 0.132868320502552f, 0.135633328591233f, 0.138431613955729f, 0.141263290050755f,
		0.144128469755705f, 0.147027265382362f, 0.149959788682454f, 0.152926150855031f, 0.155926462553701f,
		0.158960833893705f, 0.162029374458845f, 0.16513219330827f, 0.168269398983119f, 0.171441099513036f,
		0.174647402422543f, 0.17788841473729f, 0.181164242990184f, 0.184474993227387f, 0.187820771014205f,
		0.191201681440861f, 0.194617829128147f, 0.198069318232982f, 0.201556252453853f, 0.205078735036156f,
		0.208636868777438f, 0.212230756032542f, 0.215860498718652f, 0.219526198320249f, 0.223227955893977f,
		0.226965872073417f, 0.23074004707378f, 0.23455058069651f, 0.238397572333811f, 0.242281120973093f,
		0.246201325201334f, 0.250158283209375f, 0.254152092796134f, 0.258182851372752f, 0.262250655966664f,
		0.266355603225604f, 0.270497789421545f, 0.274677310454565f, 0.278894261856656f, 0.283148738795466f,
		0.287440836077983f, 0.291770648154158f, 0.296138269120463f, 0.300543792723403f, 0.304987312362961f,
		0.309468921095997f, 0.313988711639584f, 0.3185467763743f, 0.323143207347467f, 0.32777809627633f,
		0.332451534551205f, 0.337163613238559f, 0.341914423084057f, 0.346704054515559f, 0.351532597646068f,
		0.356400142276637f, 0.361306777899234f, 0.36625259369956f, 0.371237678559833f, 0.376262121061519f,
		0.381326009488037f, 0.386429431827418f, 0.39157247577492f, 0.396755228735618f, 0.401977777826949f,
		0.407240209881218f, 0.41254261144808f, 0.417885068796976f, 0.423267667919539f, 0.428690494531971f,
		0.434153634077377f, 0.439657171728079f, 0.445201192387887f, 0.450785780694349f, 0.456411021020965f,
		0.462076997479369f, 0.467783793921492f, 0.473531493941681f, 0.479320180878805f, 0.485149937818323f,
		0.491020847594331f, 0.496932992791578f, 0.502886455747457f, 0.50888131855397f, 0.514917663059676f,
		0.520995570871595f, 0.527115123357109f, 0.533276401645826f, 0.539479486631421f, 0.545724458973463f,
		0.552011399099209f, 0.558340387205378f, 0.56471150325991f, 0.571124827003694f, 0.577580437952282f,
		0.584078415397575f, 0.590618838409497f, 0.597201785837643f, 0.603827336312907f, 0.610495568249093f,
		0.617206559844509f, 0.623960389083534f, 0.630757133738175f, 0.637596871369601f, 0.644479679329661f,
		0.651405634762384f, 0.658374814605461f, 0.665387295591707f, 0.672443154250516f, 0.679542466909286f,
		0.686685309694841f, 0.693871758534824f, 0.701101889159085f, 0.708375777101046f, 0.71569349769906f,
		0.723055126097739f, 0.730460737249286f, 0.737910405914797f, 0.745404206665559f, 0.752942213884326f,
		0.760524501766589f, 0.768151144321824f, 0.775822215374732f, 0.783537788566466f, 0.791297937355839f,
		0.799102735020525f, 0.806952254658248f, 0.81484656918795f, 0.822785751350956f, 0.830769873712124f,
		0.838799008660978f, 0.846873228412837f, 0.854992605009927f, 0.863157210322481f, 0.871367116049835f,
		0.879622393721502f, 0.887923114698241f, 0.896269350173118f, 0.904661171172551f, 0.913098648557343f,
		0.921581853023715f, 0.930110855104312f, 0.938685725169219f, 0.947306533426946f, 0.955973349925421f,
		0.964686244552961f, 0.973445287039244f, 0.982250546956257f, 0.991102093719252f, 1.0f
	};

	FLinearColor() = default;
	UE_FORCEINLINE_HINT explicit FLinearColor(EForceInit)
	: R(0), G(0), B(0), A(0)
	{}
	constexpr UE_FORCEINLINE_HINT FLinearColor(float InR,float InG,float InB,float InA = 1.0f): R(InR), G(InG), B(InB), A(InA) {}
	
	/**
	 * Converts an FColor which is assumed to be in sRGB space, into linear color space.
	 * @param Color The sRGB color that needs to be converted into linear space.
	 * to get direct conversion use ReinterpretAsLinear
	 */
	constexpr UE_FORCEINLINE_HINT FLinearColor(const FColor& Color);

	CORE_API FLinearColor(const FVector3f& Vector);
	CORE_API explicit FLinearColor(const FVector3d& Vector); // Warning: keep this explicit, or FVector4f will be implicitly created from FVector3d via FLinearColor

	CORE_API FLinearColor(const FVector4f& Vector);
	CORE_API explicit FLinearColor(const FVector4d& Vector); // Warning: keep this explicit, or FVector4f will be implicitly created from FVector4d via FLinearColor
	
	// use Float16Color::GetFloats() directly
	CORE_API explicit FLinearColor(const FFloat16Color& C);

	// Serializer.
	
	friend FArchive& operator<<(FArchive& Ar,FLinearColor& Color)
	{
		return Ar << Color.R << Color.G << Color.B << Color.A;
	}

	bool Serialize( FArchive& Ar )
	{
		Ar << *this;
		return true;
	}

	friend void operator<<(FStructuredArchive::FSlot Slot, FLinearColor& Color)
	{
		FStructuredArchive::FRecord Record = Slot.EnterRecord();
		Record << SA_VALUE(TEXT("R"), Color.R) << SA_VALUE(TEXT("G"), Color.G) << SA_VALUE(TEXT("B"), Color.B) << SA_VALUE(TEXT("A"), Color.A);
	}

	bool Serialize(FStructuredArchive::FSlot Slot)
	{
		Slot << *this;
		return true;
	}

	// Conversions.
	[[nodiscard]] CORE_API FColor ToRGBE() const;

	/**
	 * Converts an FColor coming from an observed sRGB output, into a linear color.
	 * @param Color The sRGB color that needs to be converted into linear space.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT static FLinearColor FromSRGBColor(const FColor& Color)
	{
		return FLinearColor(Color);
	}

	/**
	 * Converts an FColor coming from an observed Pow(1/2.2) output, into a linear color.
	 * @param Color The Pow(1/2.2) color that needs to be converted into linear space.
	 */
	[[nodiscard]] CORE_API static FLinearColor FromPow22Color(const FColor& Color);

	// Operators.

	[[nodiscard]] inline float& Component(int32 Index)
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return RGBA[Index];
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	[[nodiscard]] inline const float& Component(int32 Index) const
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return RGBA[Index];
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	[[nodiscard]] inline FLinearColor operator+(const FLinearColor& ColorB) const
	{
		return FLinearColor(
			this->R + ColorB.R,
			this->G + ColorB.G,
			this->B + ColorB.B,
			this->A + ColorB.A
			);
	}
	inline FLinearColor& operator+=(const FLinearColor& ColorB)
	{
		R += ColorB.R;
		G += ColorB.G;
		B += ColorB.B;
		A += ColorB.A;
		return *this;
	}

	[[nodiscard]] inline FLinearColor operator-(const FLinearColor& ColorB) const
	{
		return FLinearColor(
			this->R - ColorB.R,
			this->G - ColorB.G,
			this->B - ColorB.B,
			this->A - ColorB.A
			);
	}
	inline FLinearColor& operator-=(const FLinearColor& ColorB)
	{
		R -= ColorB.R;
		G -= ColorB.G;
		B -= ColorB.B;
		A -= ColorB.A;
		return *this;
	}

	[[nodiscard]] inline FLinearColor operator*(const FLinearColor& ColorB) const
	{
		return FLinearColor(
			this->R * ColorB.R,
			this->G * ColorB.G,
			this->B * ColorB.B,
			this->A * ColorB.A
			);
	}
	inline FLinearColor& operator*=(const FLinearColor& ColorB)
	{
		R *= ColorB.R;
		G *= ColorB.G;
		B *= ColorB.B;
		A *= ColorB.A;
		return *this;
	}

	[[nodiscard]] inline FLinearColor operator*(float Scalar) const
	{
		return FLinearColor(
			this->R * Scalar,
			this->G * Scalar,
			this->B * Scalar,
			this->A * Scalar
			);
	}

	inline FLinearColor& operator*=(float Scalar)
	{
		R *= Scalar;
		G *= Scalar;
		B *= Scalar;
		A *= Scalar;
		return *this;
	}

	[[nodiscard]] inline FLinearColor operator/(const FLinearColor& ColorB) const
	{
		return FLinearColor(
			this->R / ColorB.R,
			this->G / ColorB.G,
			this->B / ColorB.B,
			this->A / ColorB.A
			);
	}
	inline FLinearColor& operator/=(const FLinearColor& ColorB)
	{
		R /= ColorB.R;
		G /= ColorB.G;
		B /= ColorB.B;
		A /= ColorB.A;
		return *this;
	}

	[[nodiscard]] inline FLinearColor operator/(float Scalar) const
	{
		const float	InvScalar = 1.0f / Scalar;
		return FLinearColor(
			this->R * InvScalar,
			this->G * InvScalar,
			this->B * InvScalar,
			this->A * InvScalar
			);
	}
	inline FLinearColor& operator/=(float Scalar)
	{
		const float	InvScalar = 1.0f / Scalar;
		R *= InvScalar;
		G *= InvScalar;
		B *= InvScalar;
		A *= InvScalar;
		return *this;
	}

	// clamped in 0..1 range
	[[nodiscard]] inline FLinearColor GetClamped(float InMin = 0.0f, float InMax = 1.0f) const
	{
		FLinearColor Ret;

		Ret.R = FMath::Clamp(R, InMin, InMax);
		Ret.G = FMath::Clamp(G, InMin, InMax);
		Ret.B = FMath::Clamp(B, InMin, InMax);
		Ret.A = FMath::Clamp(A, InMin, InMax);

		return Ret;
	}

	/** Comparison operators */
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const FLinearColor& ColorB) const
	{
		return this->R == ColorB.R && this->G == ColorB.G && this->B == ColorB.B && this->A == ColorB.A;
	}
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const FLinearColor& Other) const
	{
		return this->R != Other.R || this->G != Other.G || this->B != Other.B || this->A != Other.A;
	}

	// Error-tolerant comparison.
	[[nodiscard]] UE_FORCEINLINE_HINT bool Equals(const FLinearColor& ColorB, float Tolerance=UE_KINDA_SMALL_NUMBER) const
	{
		return FMath::Abs(this->R - ColorB.R) < Tolerance && FMath::Abs(this->G - ColorB.G) < Tolerance && FMath::Abs(this->B - ColorB.B) < Tolerance && FMath::Abs(this->A - ColorB.A) < Tolerance;
	}

	[[nodiscard]] constexpr FLinearColor CopyWithNewOpacity(float NewOpacity) const
	{
		FLinearColor NewCopy = *this;
		NewCopy.A = NewOpacity;
		return NewCopy;
	}

	/**
	 * Converts byte hue-saturation-brightness to floating point red-green-blue.
	 */
	[[nodiscard]] static CORE_API FLinearColor MakeFromHSV8(uint8 H, uint8 S, uint8 V);

	/**
	* Makes a random but quite nice color.
	*/
	[[nodiscard]] static CORE_API FLinearColor MakeRandomColor();

	/**
	* Converts temperature in Kelvins of a black body radiator to RGB chromaticity.
	*/
	[[nodiscard]] static CORE_API FLinearColor MakeFromColorTemperature( float Temp );

	/**
	* Makes a random color based on a seed.
	*/
	[[nodiscard]] static CORE_API FLinearColor MakeRandomSeededColor(int32 Seed);

	/**
	 * Euclidean distance between two points.
	 */
	[[nodiscard]] static inline float Dist( const FLinearColor &V1, const FLinearColor &V2 )
	{
		return FMath::Sqrt( FMath::Square(V2.R-V1.R) + FMath::Square(V2.G-V1.G) + FMath::Square(V2.B-V1.B) + FMath::Square(V2.A-V1.A) );
	}

	/**
	 * Generates a list of sample points on a Bezier curve defined by 2 points.
	 *
	 * @param	ControlPoints	Array of 4 Linear Colors (vert1, controlpoint1, controlpoint2, vert2).
	 * @param	NumPoints		Number of samples.
	 * @param	OutPoints		Receives the output samples.
	 * @return					Path length.
	 */
	[[nodiscard]] static CORE_API float EvaluateBezier(const FLinearColor* ControlPoints, int32 NumPoints, TArray<FLinearColor>& OutPoints);

	/** Converts a linear space RGB color to an HSV color */
	[[nodiscard]] CORE_API FLinearColor LinearRGBToHSV() const;

	/** Converts an HSV color to a linear space RGB color */
	[[nodiscard]] CORE_API FLinearColor HSVToLinearRGB() const;

	/**
	 * Linearly interpolates between two colors by the specified progress amount.  The interpolation is performed in HSV color space
	 * taking the shortest path to the new color's hue.  This can give better results than FMath::Lerp(), but is much more expensive.
	 * The incoming colors are in RGB space, and the output color will be RGB.  The alpha value will also be interpolated.
	 * 
	 * @param	From		The color and alpha to interpolate from as linear RGBA
	 * @param	To			The color and alpha to interpolate to as linear RGBA
	 * @param	Progress	Scalar interpolation amount (usually between 0.0 and 1.0 inclusive)
	 * @return	The interpolated color in linear RGB space along with the interpolated alpha value
	 */
	[[nodiscard]] static CORE_API FLinearColor LerpUsingHSV( const FLinearColor& From, const FLinearColor& To, const float Progress );

	/** Quantizes the linear color with rounding and returns the result as a FColor.  This bypasses the SRGB conversion. 
	* QuantizeRound can be dequantized back to linear with FColor::ReinterpretAsLinear (just /255.f)
	* this matches the GPU UNORM<->float conversion spec and should be preferred
	*/
	[[nodiscard]] inline FColor QuantizeRound() const;
	
	/** Quantizes the linear color and returns the result as a FColor.  This bypasses the SRGB conversion.
	* Uses floor quantization, which does not match the GPU standard conversion.
	* Restoration to float should be done with a +0.5 bias to restore to centered buckets.
	* Do NOT use this for graphics or textures or images, use QuantizeRound instead.
	*/
	[[nodiscard]] inline FColor QuantizeFloor() const;

	/** Quantizes the linear color and returns the result as a FColor with optional sRGB conversion. 
	* Clamps in [0,1] range before conversion.
	* ToFColor(false) is QuantizeRound
	*/
	[[nodiscard]] CORE_API FColor ToFColorSRGB() const;
	
	[[nodiscard]] inline FColor ToFColor(const bool bSRGB) const;
	
	/**
	 * Returns a desaturated color, with 0 meaning no desaturation and 1 == full desaturation
	 *
	 * @param	Desaturation	Desaturation factor in range [0..1]
	 * @return	Desaturated color
	 */
	[[nodiscard]] CORE_API FLinearColor Desaturate( float Desaturation ) const;

	/** Computes the perceptually weighted luminance value of a color. */
	[[nodiscard]] CORE_API float GetLuminance() const;
	
	/**
	 * Returns the maximum value in this color structure
	 *
	 * @return The maximum color channel value
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT float GetMax() const
	{
		return FMath::Max( FMath::Max( FMath::Max( R, G ), B ), A );
	}

	/** useful to detect if a light contribution needs to be rendered */
	[[nodiscard]] bool IsAlmostBlack() const
	{
		return FMath::Square(R) < UE_DELTA && FMath::Square(G) < UE_DELTA && FMath::Square(B) < UE_DELTA;
	}

	/**
	 * Returns the minimum value in this color structure
	 *
	 * @return The minimum color channel value
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT float GetMin() const
	{
		return FMath::Min( FMath::Min( FMath::Min( R, G ), B ), A );
	}

	[[nodiscard]] CORE_API FString ToString() const;

	/**
	 * Initialize this Color based on an FString. The String is expected to contain R=, G=, B=, A=.
	 * The FLinearColor will be bogus when InitFromString returns false.
	 *
	 * @param InSourceString FString containing the color values.
	 * @return true if the R,G,B values were read successfully; false otherwise.
	 */
	CORE_API bool InitFromString( const FString& InSourceString );

	/**
	 * Helper for pixel format conversions. Clamps to [0,1], mapping NaNs to 0,
	 * for consistency with GPU conversions.
	 * 
	 * @param InValue The input value.
	 * @return InValue clamped to [0,1]. NaNs map to 0.
	 */
	[[nodiscard]] static inline float Clamp01NansTo0(float InValue)
	{
		// Write this explicitly instead of using FMath::Clamp because we're particular
		// about what happens with NaNs here.
		const float ClampedLo = (InValue > 0.0f) ? InValue : 0.0f; // Also turns NaNs into 0.
		return (ClampedLo < 1.0f) ? ClampedLo : 1.0f;
	}

	/**
	 * @brief Helper function to generate distinct colors from a sequence of integers where each integer increment
	 * spins around the Hue wheel by increments of the golden ratio.
	 * @param Seed			the "seed" for a semi-random, but deterministic color
	 * @param Saturation	0-1 saturation of resulting color
	 * @param Value			0-1 brightness of resulting color
	 * @param HueRotation	0-360 amount to rotate around the hue wheel
	 * @return color with semi-random hue */
	[[nodiscard]] static FLinearColor IntToDistinctColor(
		const int32 Seed,
		const float Saturation=1.f,
		const float Value=1.f,
		const float HueRotation=180.f)
	{
		constexpr float GoldenRatio = 1.618033988749895f; // (1 + FMath::Sqrt(5.f)) / 2.f;
		return FLinearColor(static_cast<float>(Seed) * GoldenRatio * HueRotation, Saturation, Value).HSVToLinearRGB();
	}

	// Common colors.	
	static CORE_API const FLinearColor White;
	static CORE_API const FLinearColor Gray;
	static CORE_API const FLinearColor Black;
	static CORE_API const FLinearColor Transparent;
	static CORE_API const FLinearColor Red;
	static CORE_API const FLinearColor Green;
	static CORE_API const FLinearColor Blue;
	static CORE_API const FLinearColor Yellow;

	[[nodiscard]] friend UE_FORCEINLINE_HINT uint32 GetTypeHash( const FLinearColor& LinearColor )
	{
		// Note: this assumes there's no padding in FLinearColor that could contain uncompared data.
		return FCrc::MemCrc_DEPRECATED(&LinearColor, sizeof(FLinearColor));
	}
};
DECLARE_INTRINSIC_TYPE_LAYOUT(FLinearColor);

UE_FORCEINLINE_HINT FLinearColor operator*(float Scalar,const FLinearColor& Color)
{
	return Color.operator*( Scalar );
}

//
//	FColor
//	Stores a color with 8 bits of precision per channel.  
//	Note: Linear color values should always be converted to gamma space before stored in an FColor, as 8 bits of precision is not enough to store linear space colors!
//	This can be done with FLinearColor::ToFColor(true) 
//

struct FColor
{
public:
	// Variables.
#if PLATFORM_LITTLE_ENDIAN
	union { struct{ uint8 B,G,R,A; }; uint32 Bits; };
#else // PLATFORM_LITTLE_ENDIAN
	union { struct{ uint8 A,R,G,B; }; uint32 Bits; };
#endif

	uint32& DWColor(void) {return Bits;}
	const uint32& DWColor(void) const {return Bits;}

	// Constructors.
	FColor() = default;
	UE_FORCEINLINE_HINT explicit FColor(EForceInit)
	{
		// put these into the body for proper ordering with INTEL vs non-INTEL_BYTE_ORDER
		R = G = B = A = 0;
	}
	constexpr inline FColor( uint8 InR, uint8 InG, uint8 InB, uint8 InA = 255 )
		// put these into the body for proper ordering with INTEL vs non-INTEL_BYTE_ORDER
#if PLATFORM_LITTLE_ENDIAN
		: B(InB), G(InG), R(InR), A(InA)
#else
		: A(InA), R(InR), G(InG), B(InB)
#endif
	{}

	UE_FORCEINLINE_HINT explicit FColor( uint32 InColor )
	{ 
		DWColor() = InColor; 
	}

	// Serializer.
	friend FArchive& operator<< (FArchive &Ar, FColor &Color )
	{
		return Ar << Color.DWColor();
	}

	bool Serialize( FArchive& Ar )
	{
		Ar << *this;
		return true;
	}

	// Serializer.
	friend void operator<< (FStructuredArchive::FSlot Slot, FColor &Color)
	{
		return Slot << Color.DWColor();
	}

	bool Serialize(FStructuredArchive::FSlot Slot)
	{
		Slot << *this;
		return true;
	}

	// Operators.
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==( const FColor &C ) const
	{
		return DWColor() == C.DWColor();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=( const FColor& C ) const
	{
		return DWColor() != C.DWColor();
	}

	inline void operator+=(const FColor& C)
	{
		R = (uint8) FMath::Min((int32) R + (int32) C.R,255);
		G = (uint8) FMath::Min((int32) G + (int32) C.G,255);
		B = (uint8) FMath::Min((int32) B + (int32) C.B,255);
		A = (uint8) FMath::Min((int32) A + (int32) C.A,255);
	}

	[[nodiscard]] CORE_API FLinearColor FromRGBE() const;

	/**
	 * Creates a color value from the given hexadecimal string.
	 *
	 * Supported formats are: RGB, RRGGBB, RRGGBBAA, #RGB, #RRGGBB, #RRGGBBAA
	 *
	 * @param HexString - The hexadecimal string.
	 * @return The corresponding color value.
	 * @see ToHex
	 */
	[[nodiscard]] static CORE_API FColor FromHex( const FString& HexString );

	/**
	 * Makes a random but quite nice color.
	 */
	[[nodiscard]] static CORE_API FColor MakeRandomColor();

	/**
	 * Makes a color red->green with the passed in scalar (e.g. 0 is red, 1 is green)
	 */
	[[nodiscard]] static CORE_API FColor MakeRedToGreenColorFromScalar(float Scalar);

	/**
	* Converts temperature in Kelvins of a black body radiator to RGB chromaticity.
	*/
	[[nodiscard]] static CORE_API FColor MakeFromColorTemperature( float Temp );

	/**
	* Makes a random color based on a seed.
	*/
	[[nodiscard]] static CORE_API FColor MakeRandomSeededColor(int32 Seed);

	/**
	* Conversions to/from GPU UNorm floats, U8, U16
	* matches convention of FColor FLinearColor::QuantizeRound
	*/

	[[nodiscard]] static uint8 QuantizeUNormFloatTo8( float UnitFloat )
	{
		return (uint8)( 0.5f + FLinearColor::Clamp01NansTo0(UnitFloat) * 255.f );
	}
	
	[[nodiscard]] static uint16 QuantizeUNormFloatTo16( float UnitFloat )
	{
		return (uint16)( 0.5f + FLinearColor::Clamp01NansTo0(UnitFloat) * 65535.f );
	}

	[[nodiscard]] static float DequantizeUNorm8ToFloat( int Value8 )
	{
		check( Value8 >= 0 && Value8 <= 255 );

		return (float)Value8 * (1.f/255.f);
	}
	
	[[nodiscard]] static float DequantizeUNorm16ToFloat( int Value16 )
	{
		check( Value16 >= 0 && Value16 <= 65535 );

		return (float)Value16 * (1.f/65535.f);
	}

	[[nodiscard]] static uint8 Requantize10to8( int Value10 )
	{
		check( Value10 >= 0 && Value10 <= 1023 );

		// Dequantize from 10 bit (Value10/1023.f)
		// requantize to 8 bit with rounding (GPU convention UNorm)
		//  this is the computation we want :
		// (int)( (Value10/1023.f)*255.f + 0.5f );
		// this gives the exactly the same results :
		int Temp = Value10*255 + (1<<9);
		int Value8 = (Temp + (Temp >> 10)) >> 10;
		return (uint8)Value8;
	}
	
	[[nodiscard]] static uint8 Requantize16to8(int Value16)
	{
		check( Value16 >= 0 && Value16 <= 65535 );

		// Dequantize x from 16 bit (Value16/65535.f)
		// then requantize to 8 bit with rounding (GPU convention UNorm)

		// matches exactly with :
		//  (int)( (Value16/65535.f) * 255.f + 0.5f );
		int Value8 = (Value16*255 + 32895)>>16;
		return (uint8)Value8;
	}

	/**
	* Return 8-bit color Quantized from 10-bit RGB , 2-bit A
	*/
	[[nodiscard]] static FColor MakeRequantizeFrom1010102( int R, int G, int B, int A )
	{
		check( A >= 0 && A <= 3 );

		// requantize 2 bits to 8 ; could bit-replicate or just table lookup :
		const uint8 Requantize2to8[4] = { 0, 0x55, 0xAA, 0xFF };
		return FColor(
			Requantize10to8(R),
			Requantize10to8(G),
			Requantize10to8(B),
			Requantize2to8[A] );

	}

	/**
	 *	@return a new FColor based of this color with the new alpha value.
	 *	Usage: const FColor& MyColor = FColorList::Green.WithAlpha(128);
	 */
	[[nodiscard]] FColor WithAlpha( uint8 Alpha ) const
	{
		return FColor( R, G, B, Alpha );
	}

	/**
	 * Reinterprets the color as a linear color.
	 * This is the correct dequantizer for QuantizeRound.
	 * This matches the GPU spec conversion for U8<->float
	 * @return The linear color representation.
	 */
	[[nodiscard]] inline FLinearColor ReinterpretAsLinear() const
	{
		constexpr float inv255 = 1.f / 255.f;
		return FLinearColor(R * inv255, G * inv255, B * inv255, A * inv255);
	}

	/**
	 * Converts this color value to a hexadecimal string.
	 *
	 * The format of the string is RRGGBBAA.
	 *
	 * @return Hexadecimal string.
	 * @see FromHex, ToString
	 */
	[[nodiscard]] CORE_API FString ToHex() const;

	/**
	 * Converts this color value to a string.
	 *
	 * @return The string representation.
	 * @see ToHex
	 */
	[[nodiscard]] CORE_API FString ToString() const;

	/**
	 * Initialize this Color based on an FString. The String is expected to contain R=, G=, B=, A=.
	 * The FColor will be bogus when InitFromString returns false.
	 *
	 * @param	InSourceString	FString containing the color values.
	 * @return true if the R,G,B values were read successfully; false otherwise.
	 */
	CORE_API bool InitFromString( const FString& InSourceString );

	/**
	 * Gets the color in a packed uint32 format packed in the order ARGB.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT uint32 ToPackedARGB() const
	{
		return ( A << 24 ) | ( R << 16 ) | ( G << 8 ) | ( B << 0 );
	}

	/**
	 * Gets the color in a packed uint32 format packed in the order ABGR.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT uint32 ToPackedABGR() const
	{
		return ( A << 24 ) | ( B << 16 ) | ( G << 8 ) | ( R << 0 );
	}

	/**
	 * Gets the color in a packed uint32 format packed in the order RGBA.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT uint32 ToPackedRGBA() const
	{
		return ( R << 24 ) | ( G << 16 ) | ( B << 8 ) | ( A << 0 );
	}

	/**
	 * Gets the color in a packed uint32 format packed in the order BGRA.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT uint32 ToPackedBGRA() const
	{
		return ( B << 24 ) | ( G << 16 ) | ( R << 8 ) | ( A << 0 );
	}

	/** Some pre-inited colors, useful for debug code */
	static CORE_API const FColor White;
	static CORE_API const FColor Black;
	static CORE_API const FColor Transparent;
	static CORE_API const FColor Red;
	static CORE_API const FColor Green;
	static CORE_API const FColor Blue;
	static CORE_API const FColor Yellow;
	static CORE_API const FColor Cyan;
	static CORE_API const FColor Magenta;
	static CORE_API const FColor Orange;
	static CORE_API const FColor Purple;
	static CORE_API const FColor Turquoise;
	static CORE_API const FColor Silver;
	static CORE_API const FColor Emerald;

	[[nodiscard]] friend UE_FORCEINLINE_HINT uint32 GetTypeHash( const FColor& Color )
	{
		return Color.DWColor();
	}

private:
	/**
	 * Please use .ToFColor(true) on FLinearColor if you wish to convert from FLinearColor to FColor,
	 * with proper sRGB conversion applied.
	 *
	 * Note: Do not implement or make public.  We don't want people needlessly and implicitly converting between
	 * FLinearColor and FColor.  It's not a free conversion.
	 */
	explicit FColor(const FLinearColor& LinearColor);
};
DECLARE_INTRINSIC_TYPE_LAYOUT(FColor);

constexpr UE_FORCEINLINE_HINT FLinearColor::FLinearColor(const FColor& Color)
	: R(sRGBToLinearTable[Color.R])
    , G(sRGBToLinearTable[Color.G])
    , B(sRGBToLinearTable[Color.B])
    , A(static_cast<float>(Color.A) * (1.0f / 255.0f))
{
}

inline FColor FLinearColor::QuantizeRound() const
{
	// Avoid FMath::RoundToInt because it calls floor()
	return FColor(
		(uint8)(0.5f + Clamp01NansTo0(R) * 255.f),
		(uint8)(0.5f + Clamp01NansTo0(G) * 255.f),
		(uint8)(0.5f + Clamp01NansTo0(B) * 255.f),
		(uint8)(0.5f + Clamp01NansTo0(A) * 255.f)
	);
}

inline FColor FLinearColor::QuantizeFloor() const
{
	return FColor(
		(uint8)(Clamp01NansTo0(R) * 255.f),
		(uint8)(Clamp01NansTo0(G) * 255.f),
		(uint8)(Clamp01NansTo0(B) * 255.f),
		(uint8)(Clamp01NansTo0(A) * 255.f)
	);
}

inline FColor FLinearColor::ToFColor(const bool bSRGB) const
{
	if ( bSRGB )
	{
		return ToFColorSRGB();
	}
	else
	{
		return QuantizeRound();
	}
}


/** Computes a brightness and a fixed point color from a floating point color. */
extern CORE_API void ComputeAndFixedColorAndIntensity(const FLinearColor& InLinearColor,FColor& OutColor,float& OutIntensity);

/** Convert multiple FLinearColors to sRGB FColor; array version of FLinearColor::ToFColorSRGB. */
extern CORE_API void ConvertFLinearColorsToFColorSRGB(const FLinearColor* InLinearColors, FColor* OutColorsSRGB, int64 InCount);

// These act like a POD
template <> struct TIsPODType<FColor> { enum { Value = true }; };
template <> struct TIsPODType<FLinearColor> { enum { Value = true }; };


/**
 * Helper struct for a 16 bit 565 color of a DXT1/3/5 block.
 */
struct UE_DEPRECATED(5.7, "FDXTColor565 is deprecated, manipulate DXT blocks directly as required") FDXTColor565
{
	/** Blue component, 5 bit. */
	uint16 B:5;

	/** Green component, 6 bit. */
	uint16 G:6;

	/** Red component, 5 bit */
	uint16 R:5;
};


/**
 * Helper struct for a 16 bit 565 color of a DXT1/3/5 block.
 */
struct UE_DEPRECATED(5.7, "FDXTColor16 is deprecated, manipulate DXT blocks directly as required") FDXTColor16
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	union 
	{
		/** 565 Color */
		FDXTColor565 Color565;
		/** 16 bit entity representation for easy access. */
		uint16 Value;
	};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};


/**
 * Structure encompassing single DXT1 block.
 */
struct UE_DEPRECATED(5.7, "FDXT1 is deprecated, manipulate DXT blocks directly as required") FDXT1
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Color 0/1 */
	union
	{
		FDXTColor16 Color[2];
		uint32 Colors;
	};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	/** Indices controlling how to blend colors. */
	uint32 Indices;
};


/**
 * Structure encompassing single DXT5 block
 */
struct UE_DEPRECATED(5.7, "FDXT5 is deprecated, manipulate DXT blocks directly as required") FDXT5
{
	/** Alpha component of DXT5 */
	uint8	Alpha[8];
	/** DXT1 color component. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FDXT1	DXT1;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS

// Make DXT helpers act like PODs
template <> struct TIsPODType<FDXT1> { enum { Value = true }; };
template <> struct TIsPODType<FDXT5> { enum { Value = true }; };
template <> struct TIsPODType<FDXTColor16> { enum { Value = true }; };
template <> struct TIsPODType<FDXTColor565> { enum { Value = true }; };

PRAGMA_ENABLE_DEPRECATION_WARNINGS
