// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Math/Color.h"
#include "Misc/Exec.h"

class FString;

class FColorList
{
public:
	typedef TMap< FString, const FColor* > TColorsMap;
	typedef TArray< const FColor* > TColorsLookup;

	// Common colors.	
	static CORE_API const FColor White;
	static CORE_API const FColor Red;
	static CORE_API const FColor Green;
	static CORE_API const FColor Blue;
	static CORE_API const FColor Magenta;
	static CORE_API const FColor Cyan;
	static CORE_API const FColor Yellow;
	static CORE_API const FColor Black;
	static CORE_API const FColor Aquamarine;
	static CORE_API const FColor BakerChocolate;
	static CORE_API const FColor BlueViolet;
	static CORE_API const FColor Brass;
	static CORE_API const FColor BrightGold;
	static CORE_API const FColor Brown;
	static CORE_API const FColor Bronze;
	static CORE_API const FColor BronzeII;
	static CORE_API const FColor CadetBlue;
	static CORE_API const FColor CoolCopper;
	static CORE_API const FColor Copper;
	static CORE_API const FColor Coral;
	static CORE_API const FColor CornFlowerBlue;
	static CORE_API const FColor DarkBrown;
	static CORE_API const FColor DarkGreen;
	static CORE_API const FColor DarkGreenCopper;
	static CORE_API const FColor DarkOliveGreen;
	static CORE_API const FColor DarkOrchid;
	static CORE_API const FColor DarkPurple;
	static CORE_API const FColor DarkSlateBlue;
	static CORE_API const FColor DarkSlateGrey;
	static CORE_API const FColor DarkTan;
	static CORE_API const FColor DarkTurquoise;
	static CORE_API const FColor DarkWood;
	static CORE_API const FColor DimGrey;
	static CORE_API const FColor DustyRose;
	static CORE_API const FColor Feldspar;
	static CORE_API const FColor Firebrick;
	static CORE_API const FColor ForestGreen;
	static CORE_API const FColor Gold;
	static CORE_API const FColor Goldenrod;
	static CORE_API const FColor Grey;
	static CORE_API const FColor GreenCopper;
	static CORE_API const FColor GreenYellow;
	static CORE_API const FColor HunterGreen;
	static CORE_API const FColor IndianRed;
	static CORE_API const FColor Khaki;
	static CORE_API const FColor LightBlue;
	static CORE_API const FColor LightGrey;
	static CORE_API const FColor LightSteelBlue;
	static CORE_API const FColor LightWood;
	static CORE_API const FColor LimeGreen;
	static CORE_API const FColor MandarianOrange;
	static CORE_API const FColor Maroon;
	static CORE_API const FColor MediumAquamarine;
	static CORE_API const FColor MediumBlue;
	static CORE_API const FColor MediumForestGreen;
	static CORE_API const FColor MediumGoldenrod;
	static CORE_API const FColor MediumOrchid;
	static CORE_API const FColor MediumSeaGreen;
	static CORE_API const FColor MediumSlateBlue;
	static CORE_API const FColor MediumSpringGreen;
	static CORE_API const FColor MediumTurquoise;
	static CORE_API const FColor MediumVioletRed;
	static CORE_API const FColor MediumWood;
	static CORE_API const FColor MidnightBlue;
	static CORE_API const FColor NavyBlue;
	static CORE_API const FColor NeonBlue;
	static CORE_API const FColor NeonPink;
	static CORE_API const FColor NewMidnightBlue;
	static CORE_API const FColor NewTan;
	static CORE_API const FColor OldGold;
	static CORE_API const FColor Orange;
	static CORE_API const FColor OrangeRed;
	static CORE_API const FColor Orchid;
	static CORE_API const FColor PaleGreen;
	static CORE_API const FColor Pink;
	static CORE_API const FColor Plum;
	static CORE_API const FColor Quartz;
	static CORE_API const FColor RichBlue;
	static CORE_API const FColor Salmon;
	static CORE_API const FColor Scarlet;
	static CORE_API const FColor SeaGreen;
	static CORE_API const FColor SemiSweetChocolate;
	static CORE_API const FColor Sienna;
	static CORE_API const FColor Silver;
	static CORE_API const FColor SkyBlue;
	static CORE_API const FColor SlateBlue;
	static CORE_API const FColor SpicyPink;
	static CORE_API const FColor SpringGreen;
	static CORE_API const FColor SteelBlue;
	static CORE_API const FColor SummerSky;
	static CORE_API const FColor Tan;
	static CORE_API const FColor Thistle;
	static CORE_API const FColor Turquoise;
	static CORE_API const FColor VeryDarkBrown;
	static CORE_API const FColor VeryLightGrey;
	static CORE_API const FColor Violet;
	static CORE_API const FColor VioletRed;
	static CORE_API const FColor Wheat;
	static CORE_API const FColor YellowGreen;

	/** Initializes list of common colors. */
	CORE_API void CreateColorMap();

	/** Returns a color based on ColorName. If not found, returns White. */
	[[nodiscard]] CORE_API const FColor& GetFColorByName( const TCHAR* ColorName ) const;

	/** Returns a linear color based on ColorName. If not found, returns White. */
	[[nodiscard]] CORE_API const FLinearColor GetFLinearColorByName( const TCHAR* ColorName ) const;

	/** Returns true if color is valid common colors. If not found returns false. */
	[[nodiscard]] CORE_API bool IsValidColorName( const TCHAR* ColorName ) const;

	/** Returns index of color based on ColorName. If not found returns 0. */
	[[nodiscard]] CORE_API int32 GetColorIndex( const TCHAR* ColorName ) const;

	/** Returns a color based on index. If index is invalid, returns White. */
	[[nodiscard]] CORE_API const FColor& GetFColorByIndex( int32 ColorIndex ) const;

	/** Returns color's name based on index. If index is invalid, returns BadIndex. */
	[[nodiscard]] CORE_API const FString& GetColorNameByIndex( int32 ColorIndex ) const;

	/** Returns the number of colors. */
	[[nodiscard]] int32 GetColorsNum() const
	{
		return ColorsMap.Num();
	}

	/** Prints to log all colors information. */
	CORE_API void LogColors();

protected:
	CORE_API void InitializeColor( const TCHAR* ColorName, const FColor* ColorPtr, int32& CurrentIndex );

	/** List of common colors. */
	TColorsMap ColorsMap;

	/** Array of colors for fast lookup when using index. */
	TColorsLookup ColorsLookup;
};


extern CORE_API FColorList GColorList;
