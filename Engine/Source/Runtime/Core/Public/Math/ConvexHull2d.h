// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector2D.h"
#include "Math/Vector.h"

namespace ConvexHull2D
{
	/**
	 * Andrew's monotone chain convex hull algorithm for 2-dimensional points. O(N log N).
	 *
	 * Not the fastest algorithm out there, but definitely the simplest one to understand.
	 *
	 * 1 - Sort O(N log N)
	 * 2 - Scan sorted vertex from left to right to compute lower hull  O(N)
	 * 3 - Scan sorted vertex from right to left to compute upper hull. O(N)
	 *
	 * If this is slow for some reason, O(N log H) also exists where H is the number of outputted
	 * hull vertices which is normally a lot lower than N and helps reduce the overall complexity.
	 */
	template<typename VectorType, typename Allocator>
	void ComputeConvexHull(const TArray<VectorType, Allocator>& Points, TArray<int32, Allocator>& OutIndices)
	{
		// Handle case too simple for the algorithm to handle
		int32 PointsNum = Points.Num();
		if (PointsNum <= 3)
		{
			for (int32 Index = 0; Index < PointsNum; ++Index)
			{
				OutIndices.Add(Index);
			}

			return;
		}

		// Simple sorted index lookup table into immutable Points array.
		TArray<uint32, Allocator> SortedIndices;
		SortedIndices.SetNumUninitialized(PointsNum);
		for (int32 Index = 0; Index < PointsNum; ++Index)
		{
			SortedIndices[Index] = Index;
		}
	
		// Get rid of costly RangeCheck during sort by using pointer directly
		Algo::Sort(
			SortedIndices,
			[P = Points.GetData()](uint32 a, uint32 b)
			{
				// Sort in Y if X are equal
				return P[a].X == P[b].X ? P[a].Y < P[b].Y : P[a].X < P[b].X;
			}
		);

		auto IsClockwise =
			[](const VectorType& O, const VectorType& A, const VectorType& B)
			{
				return ((A.X - O.X) * (B.Y - O.Y) - (A.Y - O.Y) * (B.X - O.X)) <= 0;
			};

		int32 HullIndex = 0;
		OutIndices.SetNum(PointsNum*2);

		// Build lower hull
		for (int32 Index = 0; Index < PointsNum; ++Index) 
		{
			const VectorType& B = Points[SortedIndices[Index]];
			while (HullIndex >= 2 && IsClockwise(Points[OutIndices[HullIndex - 2]], Points[OutIndices[HullIndex - 1]], B))
			{
				--HullIndex;
			}

			OutIndices[HullIndex++] = SortedIndices[Index];
		}

		// Build upper hull
		for (int32 Index = PointsNum - 1, StartIndex = HullIndex + 1; Index > 0; --Index) 
		{
			const VectorType& B = Points[SortedIndices[Index - 1]];
			while (HullIndex >= StartIndex && IsClockwise(Points[OutIndices[HullIndex - 2]], Points[OutIndices[HullIndex - 1]], B))
			{
				--HullIndex;
			}

			OutIndices[HullIndex++] = SortedIndices[Index - 1];
		}

		OutIndices.SetNum(HullIndex - 1, EAllowShrinking::No);
	}

	/** Returns <0 if C is left of A-B */
	[[nodiscard]] inline FVector::FReal ComputeDeterminant(const FVector& A, const FVector& B, const FVector& C)
	{
		const FVector::FReal u1 = B.X - A.X;
		const FVector::FReal v1 = B.Y - A.Y;
		const FVector::FReal u2 = C.X - A.X;
		const FVector::FReal v2 = C.Y - A.Y;

		return u1 * v2 - v1 * u2;
	}

	/** Returns true if 'a' is more lower-left than 'b'. */
	[[nodiscard]] inline bool ComparePoints(const FVector& A, const FVector& B)
	{
		if (A.X < B.X)
		{
			return true;
		}

		if (A.X > B.X)
		{
			return false;
		}

		if (A.Y < B.Y)
		{
			return true;
		}

		if (A.Y > B.Y)
		{
			return false;
		}

		return false;
	}

	/** 
	 * Calculates convex hull on xy-plane of points on 'Points' and stores the indices of the resulting hull in 'OutIndices'.
	 * This code was fixed to work with duplicated vertices and precision issues.
	 * 
	 * Should be replaced by ComputeConvexHull and tested properly. Keep for backward compatibility until then.
	 */
	template<typename Allocator>
	void ComputeConvexHullLegacy(const TArray<FVector, Allocator>& Points, TArray<int32, Allocator>& OutIndices)
	{
		if (Points.Num() == 0)
		{
			// Early exit here, otherwise an invalid index will be added to the output.
			return;
		}

		// Find lower-leftmost point.
		int32 HullStart = 0;
		int32 HullEnd = 0;

		for (int32 i = 1; i < Points.Num(); ++i)
		{
			if (ComparePoints(Points[i], Points[HullStart]))
			{
				HullStart = i;
			}
			if (ComparePoints(Points[HullEnd], Points[i]))
			{
				HullEnd = i;
			}
		}

		OutIndices.Add(HullStart);

		if(HullStart == HullEnd)
		{
			// convex hull degenerated to a single point
			return;
		}

		// Gift wrap Hull.
		int32 Hull = HullStart;
		int LocalEnd = HullEnd;
		bool bGoRight = true;
		bool bFinished = false;

		// sometimes it hangs on infinite loop, repeating sequence of indices (e.g. 4,9,8,9,8,...)
		while (OutIndices.Num() <= Points.Num())
		{
			int32 NextPoint = LocalEnd;

			for (int j = 0; j < Points.Num(); ++j)
			{
				if(j == NextPoint || j == Hull)
				{
					continue;
				}

				const FVector & A = Points.GetData()[Hull];
				const FVector & B = Points.GetData()[NextPoint];
				const FVector & C = Points.GetData()[j];
				FVector::FReal Deter = ComputeDeterminant(A, B, C);

				// 0.001 Bias is to stop floating point errors, when comparing points on a straight line; KINDA_SMALL_NUMBER was slightly too small to use.
				if(Deter < -0.001)
				{
					// C is left of AB, take it
					NextPoint = j;
				}
				else if(Deter < 0.001)
				{
					if(bGoRight)
					{
						if(ComparePoints(B, C))
						{
							// we go right, take it
							NextPoint = j;
						}
					}
					else
					{
						if(ComparePoints(C, B))
						{
							// we go left, take it
							NextPoint = j;
						}
					}
				}
				else
				{
					// C is right of AB, don't take it
				}
			}

			if(NextPoint == HullEnd)
			{
				// turn around
				bGoRight = false;
				LocalEnd = HullStart;
			}
				
			if(NextPoint == HullStart)
			{
				// finish
				bFinished = true;
				break;
			}

			OutIndices.Add(NextPoint);

			Hull = NextPoint;
		}

		// clear all indices if main loop was left without finishing shape
		if (!bFinished)
		{
			OutIndices.Reset();
		}
	}

	/** Returns <0 if C is left of A-B */
	[[nodiscard]] inline FVector2D::FReal ComputeDeterminant2D(const FVector2D& A, const FVector2D& B, const FVector2D& C)
	{
		const FVector2D::FReal u1 = B.X - A.X;
		const FVector2D::FReal v1 = B.Y - A.Y;
		const FVector2D::FReal u2 = C.X - A.X;
		const FVector2D::FReal v2 = C.Y - A.Y;

		return u1 * v2 - v1 * u2;
	}

	/** 
	 * Alternate simple implementation that was found to work correctly for points that are very close together (inside the 0-1 range).
	 * 
	 * Should be replaced by ComputeConvexHull and tested properly. Keep for backward compatibility until then.
	 */
	template<typename Allocator>
	void ComputeConvexHullLegacy2(const TArray<FVector2D, Allocator>& Points, TArray<int32, Allocator>& OutIndices)
	{
		if (Points.Num() == 0)
		{
			return;
		}

		// Jarvis march implementation
		int32 LeftmostIndex = -1;
		FVector2D Leftmost(FLT_MAX, FLT_MAX);

		for (int32 PointIndex = 0; PointIndex < Points.Num(); PointIndex++)
		{
			if (Points[PointIndex].X < Leftmost.X
				|| (Points[PointIndex].X == Leftmost.X && Points[PointIndex].Y < Leftmost.Y))
			{
				LeftmostIndex = PointIndex;
				Leftmost = Points[PointIndex];
			}
		}

		int32 PointOnHullIndex = LeftmostIndex;
		int32 EndPointIndex;

		do 
		{
			OutIndices.Add(PointOnHullIndex);
			EndPointIndex = 0;

			// Find the 'leftmost' point to the line from the last hull vertex to a candidate
			for (int32 j = 1; j < Points.Num(); j++)
			{
				if (EndPointIndex == PointOnHullIndex 
					|| ComputeDeterminant2D(Points[EndPointIndex], Points[OutIndices.Last()], Points[j]) < 0)
				{
					EndPointIndex = j;
				}
			}

			PointOnHullIndex = EndPointIndex;
		} 
		while (EndPointIndex != LeftmostIndex);
	}

/*
	static void Test()
	{
		{
			TArray<FVector, TInlineAllocator<8> > In;
			In.Empty(8);

			In.Add(FVector(2, 0, 0));
			In.Add(FVector(0, 0, 0));
			In.Add(FVector(1, 0, 0));
			In.Add(FVector(3, 0, 0));

			TArray<int32, TInlineAllocator<8>> Out;
			Out.Empty(8);

			// Compute the 2d convex hull of the frustum vertices in light space
			ConvexHull2D::ComputeConvexHull(In, Out);
			check(Out.Num() == 2);
			check(Out[0] == 1);
			check(Out[1] == 3);
		}

		{
			TArray<FVector, TInlineAllocator<8> > In;
			In.Empty(8);

			In.Add(FVector(2, 1, 0));

			TArray<int32, TInlineAllocator<8>> Out;
			Out.Empty(8);

			// Compute the 2d convex hull of the frustum vertices in light space
			ConvexHull2D::ComputeConvexHull(In, Out);
			check(Out.Num() == 1);
			check(Out[0] == 0);
		}

		{
			TArray<FVector, TInlineAllocator<8> > In;
			In.Empty(8);

			In.Add(FVector(0, 0, 0));
			In.Add(FVector(1, 0, 0));
			In.Add(FVector(0, 1, 0));
			In.Add(FVector(1, 1, 0));

			TArray<int32, TInlineAllocator<8>> Out;
			Out.Empty(8);

			// Compute the 2d convex hull of the frustum vertices in light space
			ConvexHull2D::ComputeConvexHull(In, Out);
			check(Out.Num() == 4);
			check(Out[0] == 0);
			check(Out[1] == 1);
			check(Out[2] == 3);
			check(Out[3] == 2);
		}

		{
			TArray<FVector, TInlineAllocator<8> > In;
			In.Empty(8);

			In.Add(FVector(0, 0, 0));
			In.Add(FVector(1, 0, 0));
			In.Add(FVector(2, 0, 0));
			In.Add(FVector(0, 1, 0));
			In.Add(FVector(1, 1, 0));
			In.Add(FVector(0, 2, 0));
			In.Add(FVector(2, 2, 0));
			In.Add(FVector(2, 2, 0));

			TArray<int32, TInlineAllocator<8>> Out;
			Out.Empty(8);

			// Compute the 2d convex hull of the frustum vertices in light space
			ConvexHull2D::ComputeConvexHull(In, Out);
			check(Out.Num() == 4);
			check(Out[0] == 0);
			check(Out[1] == 2);
			check(Out[2] == 6);
			check(Out[3] == 5);
		}

		{
			TArray<FVector, TInlineAllocator<8> > In;
			In.Empty(8);

			In.Add(FVector(2, 0, 0));
			In.Add(FVector(3, 1, 0));
			In.Add(FVector(4, 2, 0));
			In.Add(FVector(0, 2, 0));
			In.Add(FVector(1, 3, 0));
			In.Add(FVector(2, 4, 0));
			In.Add(FVector(1, 1, 0));
			In.Add(FVector(3, 3, 0));

			TArray<int32, TInlineAllocator<8>> Out;
			Out.Empty(8);

			// Compute the 2d convex hull of the frustum vertices in light space
			ConvexHull2D::ComputeConvexHull(In, Out);
			check(Out.Num() == 4);
			check(Out[0] == 3);
			check(Out[1] == 0);
			check(Out[2] == 2);
			check(Out[3] == 5);
		}
	}
*/
}
