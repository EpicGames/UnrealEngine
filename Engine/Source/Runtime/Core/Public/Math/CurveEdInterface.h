// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Color.h"
#include "Math/InterpCurvePoint.h"

/** Interface that allows the CurveEditor to edit this type of object. */
class FCurveEdInterface
{
public:
	/** Get number of keyframes in curve. */
	virtual int32 	GetNumKeys() const { return 0; }

	/** Get number of 'sub curves' in this Curve. For example, a vector curve will have 3 sub-curves, for X, Y and Z. */
	virtual int32		GetNumSubCurves() const { return 0; }

	/**
	 * Provides the color for the sub-curve button that is present on the curve tab.
	 *
	 * @param	SubCurveIndex		The index of the sub-curve. Cannot be negative nor greater or equal to the number of sub-curves.
	 * @param	bIsSubCurveHidden	Is the curve hidden?
	 * @return						The color associated to the given sub-curve index.
	 */
	virtual FColor	GetSubCurveButtonColor(int32 SubCurveIndex, bool bIsSubCurveHidden) const { return  bIsSubCurveHidden ? FColor( 32,  0,  0) : FColor(255,  0,  0); }

	/** Get the input value for the Key with the specified index. KeyIndex must be within range ie >=0 and < NumKeys. */
	virtual float	GetKeyIn(int32 KeyIndex) { return 0.f; }

	/** 
	 *	Get the output value for the key with the specified index on the specified sub-curve. 
	 *	SubIndex must be within range ie >=0 and < NumSubCurves.
	 *	KeyIndex must be within range ie >=0 and < NumKeys.
	 */
	virtual float	GetKeyOut(int32 SubIndex, int32 KeyIndex) { return 0.f; }

	/**
	 * Provides the color for the given key at the given sub-curve.
	 *
	 * @param		SubIndex	The index of the sub-curve
	 * @param		KeyIndex	The index of the key in the sub-curve
	 * @param[in]	CurveColor	The color of the curve
	 * @return					The color that is associated the given key at the given sub-curve
	 */
	virtual FColor	GetKeyColor(int32 SubIndex, int32 KeyIndex, const FColor& CurveColor) { return CurveColor; }

	/** Evaluate a subcurve at an arbitary point. Outside the keyframe range, curves are assumed to continue their end values. */
	virtual float	EvalSub(int32 SubIndex, float InVal) { return 0.f; }

	/** 
	 *	Get the interpolation mode of the specified keyframe. This can be CIM_Constant, CIM_Linear or CIM_Curve. 
	 *	KeyIndex must be within range ie >=0 and < NumKeys.
	 */
	virtual EInterpCurveMode	GetKeyInterpMode(int32 KeyIndex) const { return CIM_Linear; }

	/** 
	 *	Get the incoming and outgoing tangent for the given subcurve and key.
	 *	SubIndex must be within range ie >=0 and < NumSubCurves.
	 *	KeyIndex must be within range ie >=0 and < NumKeys.
	 */
	virtual void	GetTangents(int32 SubIndex, int32 KeyIndex, float& ArriveTangent, float& LeaveTangent) const { ArriveTangent=0.f; LeaveTangent=0.f; }

	/** Get input range of keys. Outside this region curve continues constantly the start/end values. */
	virtual void	GetInRange(float& MinIn, float& MaxIn) const { MinIn=0.f; MaxIn=0.f; }

	/** Get overall range of output values. */
	virtual void	GetOutRange(float& MinOut, float& MaxOut) const { MinOut=0.f; MaxOut=0.f; }

	/** 
	 *	Add a new key to the curve with the specified input. Its initial value is set using EvalSub at that location. 
	 *	Returns the index of the new key.
	 */
	virtual int32		CreateNewKey(float KeyIn) { return INDEX_NONE; }

	/** 
	 *	Remove the specified key from the curve.
	 *	KeyIndex must be within range ie >=0 and < NumKeys.
	 */
	virtual void	DeleteKey(int32 KeyIndex) {}

	/** 
	 *	Set the input value of the specified Key. This may change the index of the key, so the new index of the key is retured. 
	 *	KeyIndex must be within range ie >=0 and < NumKeys.
	 */
	virtual int32		SetKeyIn(int32 KeyIndex, float NewInVal) { return KeyIndex; }

	/** 
	 *	Set the output values of the specified key.
	 *	SubIndex must be within range ie >=0 and < NumSubCurves.
	 *	KeyIndex must be within range ie >=0 and < NumKeys.
	 */
	virtual void	SetKeyOut(int32 SubIndex, int32 KeyIndex, float NewOutVal) {}

	/** 
	 *	Set the method to use for interpolating between the give keyframe and the next one.
	 *	KeyIndex must be within range ie >=0 and < NumKeys.
	 */
	virtual void	SetKeyInterpMode(int32 KeyIndex, EInterpCurveMode NewMode) {}


	/** 
	 *	Set the incoming and outgoing tangent for the given subcurve and key.
	 *	SubIndex must be within range ie >=0 and < NumSubCurves.
	 *	KeyIndex must be within range ie >=0 and < NumKeys.
	 */
	virtual void	SetTangents(int32 SubIndex, int32 KeyIndex, float ArriveTangent, float LeaveTangent) {}
};
