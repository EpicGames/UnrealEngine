// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/MathFwd.h"
#include "Math/Matrix.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"

// A high-precision floating point type, consisting of two 32-bit floats (High & Low).
// Usable where doubles (fp64) are not an option.
// See also the DoubleFloat.ush shader header.
struct FDFScalar
{
	double GetDouble() const
	{
		return static_cast<double>(High) + static_cast<double>(Low);
	}

	[[nodiscard]] FDFScalar(double In)
	{
		High = static_cast<float>(In);
		Low = static_cast<float>(In - High);
	}

	FDFScalar(float High, float Low)
		: High(High), Low(Low)
	{ }

	FDFScalar() = default;

	float High = 0;
	float Low = 0;
};

struct FDFVector2
{
	[[nodiscard]] FVector2d GetVector2d() const
	{
		return FVector2d{ X().GetDouble(), Y().GetDouble() };
	}

	template<typename TInputScalar = double>
	[[nodiscard]] FDFVector2(const UE::Math::TVector2<TInputScalar>& In)
	{
		FDFScalar X(In.X);
		FDFScalar Y(In.Y);
		High = { X.High, Y.High };
		Low = { X.Low, Y.Low };
	}

	[[nodiscard]] FDFVector2(FVector2f High, FVector2f Low)
		: High(High), Low(Low)
	{}

	[[nodiscard]] FDFVector2() = default;

	[[nodiscard]] FDFScalar X() const
	{
		return { High.X, Low.X };
	}

	[[nodiscard]] FDFScalar Y() const
	{
		return { High.Y, Low.Y };
	}

	FVector2f High{};
	FVector2f Low{};
};

struct FDFVector3
{
	[[nodiscard]] FVector3d GetVector3d() const
	{
		return FVector3d{ X().GetDouble(), Y().GetDouble(), Z().GetDouble() };
	}

	template<typename TInputScalar = double>
	FDFVector3(const UE::Math::TVector<TInputScalar>& In)
	{
		FDFScalar X(In.X);
		FDFScalar Y(In.Y);
		FDFScalar Z(In.Z);
		High = { X.High, Y.High, Z.High };
		Low = { X.Low, Y.Low, Z.Low };
	}

	FDFVector3(FVector3f High, FVector3f Low)
		: High(High), Low(Low)
	{}

	FDFVector3() = default;

	[[nodiscard]] FDFScalar X() const
	{
		return { High.X, Low.X };
	}

	[[nodiscard]] FDFScalar Y() const
	{
		return { High.Y, Low.Y };
	}

	[[nodiscard]] FDFScalar Z() const
	{
		return { High.Z, Low.Z };
	}

	FVector3f High{};
	FVector3f Low{};
};

struct FDFVector4
{
	[[nodiscard]] FVector4d GetVector4d() const
	{
		return FVector4d{ X().GetDouble(), Y().GetDouble(), Z().GetDouble(), W().GetDouble() };
	}

	template<typename TInputScalar = double>
	explicit FDFVector4(const UE::Math::TVector4<TInputScalar>& In)
	{
		FDFScalar X(In.X);
		FDFScalar Y(In.Y);
		FDFScalar Z(In.Z);
		FDFScalar W(In.W);
		High = FVector4f{ X.High, Y.High, Z.High, W.High };
		Low = FVector4f{ X.Low, Y.Low, Z.Low, W.Low };
	}

	template<typename TInputScalar = double>
	explicit FDFVector4(const UE::Math::TVector<TInputScalar>& In, TInputScalar InW)
	{
		FDFScalar X(In.X);
		FDFScalar Y(In.Y);
		FDFScalar Z(In.Z);
		FDFScalar W(InW);
		High = FVector4f{ X.High, Y.High, Z.High, W.High };
		Low = FVector4f{ X.Low, Y.Low, Z.Low, W.Low };
	}

	FDFVector4(FVector4f High, FVector4f Low)
		: High(High), Low(Low)
	{}

	FDFVector4() = default;

	[[nodiscard]] FDFScalar X() const
	{
		return { High.X, Low.X };
	}

	[[nodiscard]] FDFScalar Y() const
	{
		return { High.Y, Low.Y };
	}

	[[nodiscard]] FDFScalar Z() const
	{
		return { High.Z, Low.Z };
	}

	[[nodiscard]] FDFScalar W() const
	{
		return { High.W, Low.W };
	}

	FVector4f High{};
	FVector4f Low{};
};

// Transforms *to* absolute world space
struct FDFMatrix
{
	FDFMatrix(FMatrix44f M, FVector3f PostTranslation)
		: M(M), PostTranslation(PostTranslation)
	{}

	FDFMatrix() = default;

	FMatrix44f M{};
	FVector3f PostTranslation{}; // Added to result, *after* multiplying 'M'

	// Check if the origin of the matrix is small enough to ensure the precision >= UE_DF_MIN_PRECISION
	[[nodiscard]] CORE_API static FMatrix44f SafeCastMatrix(const FMatrix& Matrix);

	// Apply post-translation to matrix
	[[nodiscard]] CORE_API static FDFMatrix MakeToRelativeWorldMatrix(const FVector3f Origin, const FMatrix& ToWorld);
	[[nodiscard]] CORE_API static FMatrix   MakeToRelativeWorldMatrixDouble(const FVector Origin, const FMatrix& ToWorld);

	[[nodiscard]] CORE_API static FDFMatrix MakeClampedToRelativeWorldMatrix(const FVector3f Origin, const FMatrix& ToWorld);
	[[nodiscard]] CORE_API static FMatrix   MakeClampedToRelativeWorldMatrixDouble(const FVector Origin, const FMatrix& ToWorld);
};

// Transforms *from* absolute world space
struct FDFInverseMatrix
{
	FDFInverseMatrix(FMatrix44f M, FVector3f PreTranslation)
		: M(M), PreTranslation(PreTranslation)
	{}

	FDFInverseMatrix() = default;

	FMatrix44f M{};
	FVector3f PreTranslation{}; // Subtracted from input position *before* multiplying 'M'

	// Apply pre-translation to matrix
	[[nodiscard]] CORE_API static FDFInverseMatrix MakeFromRelativeWorldMatrix(const FVector3f Origin, const FMatrix& FromWorld);
	[[nodiscard]] CORE_API static FMatrix          MakeFromRelativeWorldMatrixDouble(const FVector Origin, const FMatrix& FromWorld);
};
