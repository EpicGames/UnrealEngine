// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "Math/Quat.h"
#include "Math/Transform.h"

/** Dual quaternion class */
namespace UE 
{
namespace Math 
{

template<typename T>
struct TDualQuat
{
public:
	using FReal = T;

	/** rotation or real part */
	TQuat<T> R;
	/** half trans or dual part */
	TQuat<T> D;

	// Constructors
	[[nodiscard]] TDualQuat(const TQuat<T> &InR, const TQuat<T> &InD)
		: R(InR)
		, D(InD)
	{}

	[[nodiscard]] TDualQuat(const TTransform<T> &Transform)
	{
		TVector<T> V = Transform.GetTranslation()*0.5f;
		*this = TDualQuat<T>(TQuat<T>(0, 0, 0, 1), TQuat<T>(V.X, V.Y, V.Z, 0.f)) * TDualQuat<T>(Transform.GetRotation(), TQuat<T>(0, 0, 0, 0));
	}

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	[[nodiscard]] explicit TDualQuat(const TDualQuat<FArg>& From)
		: TDualQuat<T>(TQuat<T>(From.R), TQuat<T>(From.D))
	{
	}

	/** Dual quat addition */
	[[nodiscard]] TDualQuat<T> operator+(const TDualQuat<T> &B) const
	{
		return{ R + B.R, D + B.D };
	}

	/** Dual quat product */
	[[nodiscard]] TDualQuat<T> operator*(const TDualQuat<T> &B) const
	{
		return{ R*B.R, D*B.R + B.D*R };
	}

	/** Scale dual quat */
	[[nodiscard]] TDualQuat<T> operator*(const T S) const
	{
		return{ R*S, D*S };
	}

	/** Return normalized dual quat */
	[[nodiscard]] TDualQuat<T> Normalized() const
	{
		T MinV = 1.0f / FMath::Sqrt(R | R);
		return{ R*MinV, D*MinV };
	}

	/** Convert dual quat to transform */
	[[nodiscard]] TTransform<T> AsFTransform(TVector<T> Scale = TVector<T>(1.0f, 1.0f, 1.0f)) const
	{
		TQuat<T> TQ = D*TQuat<T>(-R.X, -R.Y, -R.Z, R.W);
		return TTransform<T>(R, TVector<T>(TQ.X, TQ.Y, TQ.Z)*2.0f, Scale);
	}
};

}	// namespace UE::Math
}	// namespace UE

UE_DECLARE_LWC_TYPE(DualQuat, 4);

template<> struct TIsUECoreVariant<FDualQuat4f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FDualQuat4d> { enum { Value = true }; };
