// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector.h"

/**
 * Implements an edge consisting of two vertices.
 */
struct FEdge
{
	/** Holds the edge vertices. */
	FVector Vertex[2];

	/** Holds a temporary variable used when creating arrays of unique edges. */
	int32 Count;

public:

	/** Default constructor (no initialization). */
	[[nodiscard]] FEdge() = default;

	/**
	 * Creates and initializes a new edge from two vertices.
	 *
	 * @param V1 The first vertex.
	 * @param V2 The second vertex.
	 */
	[[nodiscard]] FEdge(FVector V1, FVector V2)
	{
		Vertex[0] = V1;
		Vertex[1] = V2;
		Count = 0;
	}

public:

	/**
	 * Compares this edge with another.
	 *
	 * @param E The other edge.
	 * @return true if the two edges are identical, false otherwise.
	 */
	[[nodiscard]] bool operator==(const FEdge& E) const
	{
		return (((E.Vertex[0] == Vertex[0]) && (E.Vertex[1] == Vertex[1])) || ((E.Vertex[0] == Vertex[1]) && (E.Vertex[1] == Vertex[0])));
	}
};
