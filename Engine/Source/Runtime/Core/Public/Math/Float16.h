// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Serialization/Archive.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Float32.h"
#include "Serialization/MemoryLayout.h"

template <typename T> struct TCanBulkSerialize;

/**
* 16 bit float components and conversion
*
*
* IEEE float 16
* Represented by 10-bit mantissa M, 5-bit exponent E, and 1-bit sign S
*
* Specials:
* 
* E=0, M=0			== 0.0
* E=0, M!=0			== Denormalized value (M / 2^10) * 2^-14
* 0<E<31, M=any		== (1 + M / 2^10) * 2^(E-15)
* E=31, M=0			== Infinity
* E=31, M!=0		== NAN
* 
* conversion from 32 bit float is with RTNE (round to nearest even)
*
* Legacy code truncated in the conversion.  SetTruncate can be used for backwards compatibility.
* 
*/
class FFloat16
{
public:

	/* Float16 can store values in [-MaxF16Float,MaxF16Float] */
	constexpr static float MaxF16Float = 65504.f;

	uint16 Encoded = 0;

	/** Default constructor */
	[[nodiscard]] FFloat16() = default;

	/** Copy constructor. */
	[[nodiscard]] FFloat16(const FFloat16& FP16Value) = default;

	/** Conversion constructor. Convert from Fp32 to Fp16. */
	[[nodiscard]] FFloat16(float FP32Value);

	/** Assignment operator. Convert from Fp32 to Fp16. */
	FFloat16& operator=(float FP32Value);

	/** Assignment operator. Copy Fp16 value. */
	FFloat16& operator=(const FFloat16& FP16Value) = default;

	/** Convert from Fp16 to Fp32. */
	operator float() const;

	/** Convert from Fp32 to Fp16, round-to-nearest-even. (RTNE)
	Stores values out of range as +-Inf */
	void Set(float FP32Value);
	
	/*Convert from Fp32 to Fp16, round-to-nearest-even. (RTNE)
	Clamps values out of range as +-MaxF16Float */
	void SetClamped(float FP32Value)
	{
		Set( FMath::Clamp(FP32Value,-MaxF16Float,MaxF16Float) );
	}

	/** Convert from Fp32 to Fp16, truncating low bits. 
	(backward-compatible conversion; was used by Set() previously)
	Clamps values out of range to [-MaxF16Float,MaxF16Float] */
	void CORE_API SetTruncate(float FP32Value);

	/** Set to 0.0 **/
	void SetZero()
	{
		Encoded = 0;
	}
	
	/** Set to 1.0 **/
	void SetOne()
	{
		Encoded = 0x3c00;
	}

	/** Return float clamp in [0,MaxF16Float] , no negatives or infinites or nans returned **/
	[[nodiscard]] FFloat16 GetClampedNonNegativeAndFinite() const;
	
	/** Return float clamp in [-MaxF16Float,MaxF16Float] , no infinites or nans returned **/
	[[nodiscard]] FFloat16 GetClampedFinite() const;

	/** Convert from Fp16 to Fp32. */
	[[nodiscard]] float GetFloat() const;

	/** Is the float negative without converting
	NOTE: returns true for negative zero! */
	[[nodiscard]] bool IsNegative() const
	{
		// negative if sign bit is on
		// can be tested with int compare
		return (int16)Encoded < 0;
	}

	/**
	 * Serializes the FFloat16.
	 *
	 * @param Ar Reference to the serialization archive.
	 * @param V Reference to the FFloat16 being serialized.
	 *
	 * @return Reference to the Archive after serialization.
	 */
	friend FArchive& operator<<(FArchive& Ar, FFloat16& V)
	{
		return Ar << V.Encoded;
	}
};
template<> struct TCanBulkSerialize<FFloat16> { enum { Value = true }; };

DECLARE_INTRINSIC_TYPE_LAYOUT(FFloat16);

UE_FORCEINLINE_HINT FFloat16::FFloat16(float FP32Value)
{
	Set(FP32Value);
}	


inline FFloat16& FFloat16::operator=(float FP32Value)
{
	Set(FP32Value);
	return *this;
}


UE_FORCEINLINE_HINT FFloat16::operator float() const
{
	return GetFloat();
}


// NOTE: Set() on values out of F16 max range store them as +-Inf
UE_FORCEINLINE_HINT void FFloat16::Set(float FP32Value)
{
	// FPlatformMath::StoreHalf follows RTNE (round-to-nearest-even) rounding default convention
	FPlatformMath::StoreHalf(&Encoded, FP32Value);
}



UE_FORCEINLINE_HINT float FFloat16::GetFloat() const
{
	return FPlatformMath::LoadHalf(&Encoded);
}


/** Return float clamp in [0,MaxF16Float] , no negatives or infinites or nans returned **/
inline FFloat16 FFloat16::GetClampedNonNegativeAndFinite() const
{
	FFloat16 ReturnValue;
	
	if ( Encoded < 0x7c00 ) // normal and non-negative, just pass through
		ReturnValue.Encoded = Encoded;
	else if ( Encoded == 0x7c00 ) // infinity turns into largest normal
		ReturnValue.Encoded = 0x7bff;
	else // NaNs or anything negative turns into 0
		ReturnValue.Encoded = 0;

	return ReturnValue;
}


/** Return float clamp in [-MaxF16Float,MaxF16Float] , no infinites or nans returned **/
inline FFloat16 FFloat16::GetClampedFinite() const
{	
	FFloat16 ReturnValue;

	if ( (Encoded&0x7c00) == 0x7c00 )
	{
		// inf or nan
		if ( Encoded == 0x7C00 ) //+inf
		{
			ReturnValue.Encoded = 0x7bff; // max finite
		}
		else if ( Encoded == 0xFC00 ) //-inf
		{
			ReturnValue.Encoded = 0xfbff; // max finite negative
		}
		else
		{
			// nan
			ReturnValue.Encoded = 0;
		}
	}
	else
	{
		ReturnValue.Encoded = Encoded;
	}
	
	return ReturnValue;
}
