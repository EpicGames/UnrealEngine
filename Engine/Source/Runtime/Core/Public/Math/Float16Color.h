// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Color.h"
#include "Math/Float16.h"

/**
 *	RGBA Color made up of FFloat16
 */
class FFloat16Color
{
public:

	FFloat16 R;
	FFloat16 G;
	FFloat16 B;
	FFloat16 A;

	/* Get as a pointer to four half floats */
	[[nodiscard]] uint16 * GetFourHalves()
	{
		return (uint16 *)this;
	}
	[[nodiscard]] const uint16 * GetFourHalves() const
	{
		return (const uint16 *)this;
	}

	[[nodiscard]] const FLinearColor GetFloats() const;

	/** Default constructor */
	[[nodiscard]] FFloat16Color();

	/** Copy constructor. */
	[[nodiscard]] FFloat16Color(const FFloat16Color& Src);

	/** Constructor from a linear color. */
	[[nodiscard]] FFloat16Color(const FLinearColor& Src);

	/** assignment operator */
	FFloat16Color& operator=(const FFloat16Color& Src);

 	/**
	 * Checks whether two colors are identical.
	 *
	 * @param Src The other color.
	 * @return true if the two colors are identical, otherwise false.
	 */
	[[nodiscard]] bool operator==(const FFloat16Color& Src) const;
};


UE_FORCEINLINE_HINT FFloat16Color::FFloat16Color() { }


inline FFloat16Color::FFloat16Color(const FFloat16Color& Src)
{
	R = Src.R;
	G = Src.G;
	B = Src.B;
	A = Src.A;
}


UE_FORCEINLINE_HINT FFloat16Color::FFloat16Color(const FLinearColor& Src)
{
	FPlatformMath::VectorStoreHalf( GetFourHalves(), (const float *)&Src );
}

	
inline const FLinearColor FFloat16Color::GetFloats() const
{
	FLinearColor Ret;
	FPlatformMath::VectorLoadHalf( (float *)&Ret, GetFourHalves() );
	return Ret;
}


inline FFloat16Color& FFloat16Color::operator=(const FFloat16Color& Src)
{
	R = Src.R;
	G = Src.G;
	B = Src.B;
	A = Src.A;
	return *this;
}

inline bool FFloat16Color::operator==(const FFloat16Color& Src) const
{
	return (
		(R == Src.R) &&
		(G == Src.G) &&
		(B == Src.B) &&
		(A == Src.A)
		);
}
