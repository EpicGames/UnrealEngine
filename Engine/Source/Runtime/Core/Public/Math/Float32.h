// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
* 32 bit float components
*/
class FFloat32
{
public:

	union
	{
		struct
		{
#if PLATFORM_LITTLE_ENDIAN
			uint32	Mantissa : 23;
			uint32	Exponent : 8;
			uint32	Sign : 1;			
#else
			uint32	Sign : 1;
			uint32	Exponent : 8;
			uint32	Mantissa : 23;			
#endif
		} Components;

		float	FloatValue;
	};

	/**
	 * Constructor
	 * 
	 * @param InValue value of the float.
	 */
	FFloat32(float InValue = 0.0f);
};


UE_FORCEINLINE_HINT FFloat32::FFloat32(float InValue)
	: FloatValue(InValue)
{ }
