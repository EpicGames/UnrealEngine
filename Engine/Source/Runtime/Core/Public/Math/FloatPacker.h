// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogFloatPacker, Log, All);


/**
 *
 */
class FFloatInfo_IEEE32
{
public:
	enum { MantissaBits	= 23 };
	enum { ExponentBits	= 8 };
	enum { SignShift	= 31 };
	enum { ExponentBias	= 127 };

	enum { MantissaMask	= 0x007fffff };
	enum { ExponentMask	= 0x7f800000 };
	enum { SignMask		= 0x80000000 };

	typedef float	FloatType;
	typedef uint32	PackedType;

	[[nodiscard]] static PackedType ToPackedType(FloatType Value)
	{
		return *(PackedType*)&Value;
	}

	[[nodiscard]] static FloatType ToFloatType(PackedType Value)
	{
		return *(FloatType*)&Value;
	}
};


/**
 *
 */
template<uint32 NumExponentBits, uint32 NumMantissaBits, bool bRound, typename FloatInfo=FFloatInfo_IEEE32>
class TFloatPacker
{
public:
	enum { NumOutputsBits	= NumExponentBits + NumMantissaBits + 1			};

	enum { MantissaShift	= FloatInfo::MantissaBits - NumMantissaBits	};
	enum { ExponentBias		= (1 << (NumExponentBits-1)) - 1				};
	enum { SignShift		= NumExponentBits + NumMantissaBits				};

	enum { MantissaMask		= (1 << NumMantissaBits) - 1					};
	enum { ExponentMask		= ((1 << NumExponentBits)-1) << NumMantissaBits	};
	enum { SignMask			=  1 << SignShift								};

	enum { MinExponent		= -ExponentBias - 1								};
	enum { MaxExponent		= ExponentBias									};

	typedef typename FloatInfo::PackedType	PackedType;
	typedef typename FloatInfo::FloatType	FloatType;

	PackedType Encode(FloatType Value) const
	{
		if ( Value == (FloatType) 0.0 )
		{
			return (PackedType) 0;
		}

		const PackedType ValuePacked = FloatInfo::ToPackedType( Value );

		// Extract mantissa, exponent, sign.
				PackedType	Mantissa	= ValuePacked & FloatInfo::MantissaMask;
				int32			Exponent	= (ValuePacked & FloatInfo::ExponentMask) >> FloatInfo::MantissaBits;
		const	PackedType	Sign		= ValuePacked >> FloatInfo::SignShift;

		// Subtract IEEE's bias.
		Exponent -= FloatInfo::ExponentBias;

		if ( bRound )
		{
			Mantissa += (1 << (MantissaShift-1));
			if ( Mantissa & (1 << FloatInfo::MantissaBits) )
			{
				Mantissa = 0;
				++Exponent;
			}
		}

		// Shift the mantissa to the right
		Mantissa >>= MantissaShift;

		//UE_LOGF(LogFloatPacker, Log,  "fp: exp: %i (%i,%i)", Exponent, (int32)MinExponent, (int32)MaxExponent );
		if ( Exponent < MinExponent )
		{
			return (PackedType) 0;
		}
		if ( Exponent > MaxExponent )
		{
			Exponent = MaxExponent;
		}

		// Add our bias.
		Exponent -= MinExponent;

		return (Sign << SignShift) | (Exponent << NumMantissaBits) | (Mantissa);
	}

	[[nodiscard]] FloatType Decode(PackedType Value) const
	{
		if ( Value == (PackedType) 0 )
		{
			return (FloatType) 0.0;
		}

		// Extract mantissa, exponent, sign.
				PackedType	Mantissa	= Value & MantissaMask;
				int32			Exponent	= (Value & ExponentMask) >> NumMantissaBits;
		const	PackedType	Sign		= Value >> SignShift;

		// Subtract our bias.
		Exponent += MinExponent;
		// Add IEEE's bias.
		Exponent += FloatInfo::ExponentBias;

		Mantissa <<= MantissaShift;

		return FloatInfo::ToFloatType( (Sign << FloatInfo::SignShift) | (Exponent << FloatInfo::MantissaBits) | (Mantissa) );
	}
#if 0
	PackedType EncodeNoSign(FloatType Value)
	{
		if ( Value == (FloatType) 0.0 )
		{
			return (PackedType) 0;
		}

		const PackedType ValuePacked = FloatInfo::ToPackedType( Value );

		// Extract mantissa, exponent, sign.
				PackedType	Mantissa	= ValuePacked & FloatInfo::MantissaMask;
				int32			Exponent	= (ValuePacked & FloatInfo::ExponentMask) >> FloatInfo::MantissaBits;
		//const	PackedType	Sign		= ValuePacked >> FloatInfo::SignShift;

		// Subtract IEEE's bias.
		Exponent -= FloatInfo::ExponentBias;

		if ( bRound )
		{
			Mantissa += (1 << (MantissaShift-1));
			if ( Mantissa & (1 << FloatInfo::MantissaBits) )
			{
				Mantissa = 0;
				++Exponent;
			}
		}

		// Shift the mantissa to the right
		Mantissa >>= MantissaShift;

		//UE_LOGF(LogFloatPacker, Log,  "fp: exp: %i (%i,%i)", Exponent, (int32)MinExponent, (int32)MaxExponent );
		if ( Exponent < MinExponent )
		{
			if ( Exponent < MinExponent-1 )
			{
				return (PackedType) 0;
			}
			Exponent = MinExponent;
		}
		if ( Exponent > MaxExponent )
		{
			Exponent = MaxExponent;
		}

		// Add our bias.
		Exponent -= MinExponent;

		return (Exponent << NumMantissaBits) | (Mantissa);
	}

	FloatType DecodeNoSign(PackedType Value)
	{
		if ( Value == (PackedType) 0 )
		{
			return (FloatType) 0.0;
		}

		// Extract mantissa, exponent, sign.
		PackedType	Mantissa	= Value & MantissaMask;
		int32			Exponent	= (Value & ExponentMask) >> NumMantissaBits;
		//const	PackedType	Sign		= Value >> SignShift;

		// Subtract our bias.
		Exponent += MinExponent;
		// Add IEEE's bias.
		Exponent += FloatInfo::ExponentBias;

		Mantissa <<= MantissaShift;

		return FloatInfo::ToFloatType( (Exponent << FloatInfo::MantissaBits) | (Mantissa) );
	}
#endif
};
