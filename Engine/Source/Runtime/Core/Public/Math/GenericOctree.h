// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GenericOctree.h: Generic octree definition.
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreGlobals.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "GenericOctreePublic.h"
#include "HAL/PlatformMisc.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Box.h"
#include "Math/BoxSphereBounds.h"
#include "Math/MathFwd.h"
#include "Math/UnrealMathSSE.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Math/VectorRegister.h"
#include "Misc/AssertionMacros.h"
#include "Templates/EnableIf.h"
#include "Templates/Models.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "Trace/Detail/Channel.h"

/** A concise iteration over the children of an octree node. */
#define FOREACH_OCTREE_CHILD_NODE(ChildRef) \
	for(FOctreeChildNodeRef ChildRef(0);!ChildRef.IsNULL();ChildRef.Advance())

class FOctreeChildNodeRef;

/** An unquantized bounding box. */
class FBoxCenterAndExtent
{
public:
	using FReal = FVector4::FReal;

	FVector4 Center;	// LWC_TODO: Case for FVector4d
	FVector4 Extent;

	/** Default constructor. */
	[[nodiscard]] FBoxCenterAndExtent() {}

	/** Initialization constructor. */
	[[nodiscard]] FBoxCenterAndExtent(const FVector& InCenter,const FVector& InExtent)
	:	Center(InCenter,0)
	,	Extent(InExtent,0)
	{}

	/** FBox conversion constructor. */
	[[nodiscard]] FBoxCenterAndExtent(const FBox& Box)
	{
		FVector C, E;
		Box.GetCenterAndExtents(C,E);	// LWC_TODO: Perf pessimization
		Center = FVector4(C, 0); 
		Extent = FVector4(E, 0);
	}

	/** FBoxSphereBounds conversion constructor. */
	template<typename TExtent>
	[[nodiscard]] explicit FBoxCenterAndExtent(const UE::Math::TBoxSphereBounds<FReal, TExtent>& BoxSphere)
	{
		Center = BoxSphere.Origin;
		Extent = FVector(BoxSphere.BoxExtent);
		Center.W = Extent.W = 0;
	}

	/** Center - radius as four contiguous floats conversion constructor. */
	[[nodiscard]] explicit FBoxCenterAndExtent(const float PositionRadius[4])
	{
		Center = FVector4(PositionRadius[0],PositionRadius[1],PositionRadius[2], 0);
		Extent = FVector4(PositionRadius[3],PositionRadius[3],PositionRadius[3], 0);
	}

	/** Converts to a FBox. */
	[[nodiscard]] FBox GetBox() const
	{
		return FBox(Center - Extent,Center + Extent);
	}


	/** Comparison operator for FBoxCenterAndExtent */
	[[nodiscard]] bool operator==(const FBoxCenterAndExtent& Other) const
	{
		return Center == Other.Center && Extent == Other.Extent;
	}

	/**
	 * Determines whether two boxes intersect.
	 * Warning: this operates on the W of the bounds positions!
	 * @return true if the boxes intersect, or false.
	 */
	[[nodiscard]] friend inline bool Intersect(const FBoxCenterAndExtent& A,const FBoxCenterAndExtent& B)
	{
		// CenterDifference is the vector between the centers of the bounding boxes.
		const VectorRegister CenterDifference = VectorAbs(VectorSubtract(VectorLoadAligned(&A.Center),VectorLoadAligned(&B.Center)));

		// CompositeExtent is the extent of the bounding box which is the convolution of A with B.
		const VectorRegister CompositeExtent = VectorAdd(VectorLoadAligned(&A.Extent),VectorLoadAligned(&B.Extent));

		// For each axis, the boxes intersect on that axis if the projected distance between their centers is less than the sum of their
		// extents.  If the boxes don't intersect on any of the axes, they don't intersect.
		return VectorAnyGreaterThan(CenterDifference,CompositeExtent) == false;
	}
	/**
	 * Determines whether two boxes intersect.
	 * Warning: this operates on the W of the bounds positions!
	 * @return true if the boxes intersect, or false.
	 */
	template<typename TExtent>
	[[nodiscard]] friend inline bool Intersect(const UE::Math::TBoxSphereBounds<FReal, TExtent>& A,const FBoxCenterAndExtent& B)
	{
		// CenterDifference is the vector between the centers of the bounding boxes.
		const VectorRegister CenterDifference = VectorAbs(VectorSubtract(VectorLoadFloat3_W0(&A.Origin),VectorLoadAligned(&B.Center)));

		// CompositeExtent is the extent of the bounding box which is the convolution of A with B.
		const VectorRegister CompositeExtent = VectorAdd(VectorLoadFloat3_W0(&A.BoxExtent),VectorLoadAligned(&B.Extent));

		// For each axis, the boxes intersect on that axis if the projected distance between their centers is less than the sum of their
		// extents.  If the boxes don't intersect on any of the axes, they don't intersect.
		return VectorAnyGreaterThan(CenterDifference,CompositeExtent) == false;
	}
	/**
	 * Determines whether two boxes intersect.
	 * Warning: this operates on the W of the bounds positions!
	 * @param A box given in center - radius form as four contiguous floats
	 * @return true if the boxes intersect, or false.
	 */
	[[nodiscard]] friend inline bool Intersect(const float A[4],const FBoxCenterAndExtent& B)
	{
		// CenterDifference is the vector between the centers of the bounding boxes.
		const VectorRegister CenterDifference = VectorAbs(VectorSubtract(VectorLoadFloat3_W0(A),VectorLoadAligned(&B.Center)));

		// CompositeExtent is the extent of the bounding box which is the convolution of A with B.
		const VectorRegister CompositeExtent = VectorAdd(VectorSet_W0(VectorLoadFloat1(A+3)),VectorLoadAligned(&B.Extent));

		// For each axis, the boxes intersect on that axis if the projected distance between their centers is less than the sum of their
		// extents.  If the boxes don't intersect on any of the axes, they don't intersect.
		return VectorAnyGreaterThan(CenterDifference,CompositeExtent) == false;
	}

	[[nodiscard]] friend inline uint32 GetTypeHash(const FBoxCenterAndExtent& Box)
	{
		return HashCombine(GetTypeHash(Box.Center), GetTypeHash(Box.Extent));
	}
};

/** A reference to a child of an octree node. */
class FOctreeChildNodeRef
{
public:
	int8 Index;

	/** Initialization constructor. */
	FOctreeChildNodeRef(int8 InX, int8 InY, int8 InZ)
	{
		checkSlow(InX >= 0 && InX <= 1);
		checkSlow(InY >= 0 && InY <= 1);
		checkSlow(InZ >= 0 && InZ <= 1);
		Index = int8(InX << 0) | int8(InY << 1) | int8(InZ << 2);
	}

	/** Initialized the reference with a child index. */
	FOctreeChildNodeRef(int8 InIndex = 0)
	:	Index(InIndex)
	{
		checkSlow(Index < 8);
	}

	/** Advances the reference to the next child node.  If this was the last node remain, Index will be 8 which represents null. */
	UE_FORCEINLINE_HINT void Advance()
	{
		++Index;
	}

	/** @return true if the reference isn't set. */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsNULL() const
	{
		return Index >= 8;
	}

	UE_FORCEINLINE_HINT void SetNULL()
	{
		Index = 8;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT int32 X() const
	{
		return (Index >> 0) & 1;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT int32 Y() const
	{
		return (Index >> 1) & 1;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT int32 Z() const
	{
		return (Index >> 2) & 1;
	}
};

/** A subset of an octree node's children that intersect a bounding box. */
class FOctreeChildNodeSubset
{
public:

	union
	{
		struct 
		{
			uint32 bPositiveX : 1;
			uint32 bPositiveY : 1;
			uint32 bPositiveZ : 1;
			uint32 bNegativeX : 1;
			uint32 bNegativeY : 1;
			uint32 bNegativeZ : 1;
		};

		struct
		{
			/** Only the bits for the children on the positive side of the splits. */
			uint32 PositiveChildBits : 3;

			/** Only the bits for the children on the negative side of the splits. */
			uint32 NegativeChildBits : 3;
		};

		/** All the bits corresponding to the child bits. */
		uint32 ChildBits : 6;

		/** All the bits used to store the subset. */
		uint32 AllBits;
	};

	/** Initializes the subset to be empty. */
	FOctreeChildNodeSubset()
	:	AllBits(0)
	{}

	/** Initializes the subset to contain a single node. */
	FOctreeChildNodeSubset(FOctreeChildNodeRef ChildRef)
	:	AllBits(0)
	{
		// The positive child bits correspond to the child index, and the negative to the NOT of the child index.
		PositiveChildBits = ChildRef.Index;
		NegativeChildBits = ~ChildRef.Index;
	}

	/** Determines whether the subset contains a specific node. */
	[[nodiscard]] bool Contains(FOctreeChildNodeRef ChildRef) const;
};

/** The context of an octree node, derived from the traversal of the tree. */
class FOctreeNodeContext
{
public:

	using FReal = FVector4::FReal;

	/** The node bounds are expanded by their extent divided by LoosenessDenominator. */
	enum { LoosenessDenominator = 16 };

	/** The bounds of the node. */
	FBoxCenterAndExtent Bounds;

	/** The extent of the node's children. */
	FReal ChildExtent;

	/** The offset of the childrens' centers from the center of this node. */
	FReal ChildCenterOffset;

	/** Bits used for culling, semantics left up to the caller (except that it is always set to zero at the root). This does not consume storage because it is leftover in the padding.*/
	uint32 InCullBits;

	/** Bits used for culling, semantics left up to the caller (except that it is always set to zero at the root). This does not consume storage because it is leftover in the padding.*/
	uint32 OutCullBits;

	/** Default constructor. */
	FOctreeNodeContext()
	{}

	/** Initialization constructor, this one is used when we done care about the box anymore */
	FOctreeNodeContext(uint32 InInCullBits, uint32 InOutCullBits)
		:	InCullBits(InInCullBits)
		,	OutCullBits(InOutCullBits)
	{
	}

	/** Initialization constructor. */
	FOctreeNodeContext(const FBoxCenterAndExtent& InBounds)
	:	Bounds(InBounds)
	{
		// A child node's tight extents are half its parent's extents, and its loose extents are expanded by 1/LoosenessDenominator.
		const FReal TightChildExtent = Bounds.Extent.X * 0.5f;
		const FReal LooseChildExtent = TightChildExtent * (1.0f + 1.0f / (FReal)LoosenessDenominator);

		ChildExtent = LooseChildExtent;
		ChildCenterOffset = Bounds.Extent.X - LooseChildExtent;
	}

	/** Initialization constructor. */
	FOctreeNodeContext(const FBoxCenterAndExtent& InBounds, uint32 InInCullBits, uint32 InOutCullBits)
		:	Bounds(InBounds)
		,	InCullBits(InInCullBits)
		,	OutCullBits(InOutCullBits)
	{
		// A child node's tight extents are half its parent's extents, and its loose extents are expanded by 1/LoosenessDenominator.
		const FReal TightChildExtent = Bounds.Extent.X * 0.5f;
		const FReal LooseChildExtent = TightChildExtent * (1.0f + 1.0f / (FReal)LoosenessDenominator);

		ChildExtent = LooseChildExtent;
		ChildCenterOffset = Bounds.Extent.X - LooseChildExtent;
	}

	[[nodiscard]] inline VectorRegister GetChildOffsetVec(int i) const
	{
		// LWC_TODO: not sure this is correct for VectorRegister = VectorRegister4Double
		union MaskType { VectorRegister4Float v;  VectorRegister4Int i; 
		} Mask;

		Mask.v = MakeVectorRegister(1u, 2u, 4u, 8u);
		VectorRegister4Int X = VectorIntLoad1(&i);
		VectorRegister4Int A = VectorIntAnd(X, Mask.i);
		Mask.i = VectorIntCompareEQ(Mask.i, A);
		return VectorSelect(Mask.v, VectorSetFloat1(ChildCenterOffset), VectorSetFloat1(-ChildCenterOffset));
	}

	/** Child node initialization constructor. */
	[[nodiscard]] inline FOctreeNodeContext GetChildContext(FOctreeChildNodeRef ChildRef) const
	{
		FBoxCenterAndExtent LocalBounds;
		VectorRegister ZeroW = MakeVectorRegister(1.0f, 1.0f, 1.0f, 0.0f);
		VectorStoreAligned(VectorMultiply(ZeroW, VectorAdd(VectorLoadAligned(&Bounds.Center), GetChildOffsetVec(ChildRef.Index))), &(LocalBounds.Center.X));
		VectorStoreAligned(VectorMultiply(ZeroW, VectorSetFloat1(ChildExtent)), &(LocalBounds.Extent.X));
		return FOctreeNodeContext(LocalBounds);
	}

	/** Construct a child context given the child ref. Optimized to remove all LHS. */
	inline void GetChildContext(FOctreeChildNodeRef ChildRef, FOctreeNodeContext* ChildContext) const
	{
		VectorRegister ZeroW = MakeVectorRegister(1.0f, 1.0f, 1.0f, 0.0f);
		VectorStoreAligned(VectorMultiply(ZeroW, VectorAdd(VectorLoadAligned(&Bounds.Center), GetChildOffsetVec(ChildRef.Index))), &(ChildContext->Bounds.Center.X));
		VectorStoreAligned(VectorMultiply(ZeroW, VectorSetFloat1(ChildExtent)), &(ChildContext->Bounds.Extent.X));

		const FReal TightChildExtent = ChildExtent * 0.5f;
		const FReal LooseChildExtent = TightChildExtent * (1.0f + 1.0f / (FReal)LoosenessDenominator);
		ChildContext->ChildExtent = LooseChildExtent;
		ChildContext->ChildCenterOffset = ChildExtent - LooseChildExtent;
	}
	
	/** Child node initialization constructor. */
	[[nodiscard]] inline FOctreeNodeContext GetChildContext(FOctreeChildNodeRef ChildRef, uint32 InInCullBits, uint32 InOutCullBits) const
	{
		FBoxCenterAndExtent LocalBounds;
		VectorRegister ZeroW = MakeVectorRegister(1.0f, 1.0f, 1.0f, 0.0f);
		VectorStoreAligned(VectorMultiply(ZeroW, VectorAdd(VectorLoadAligned(&Bounds.Center), GetChildOffsetVec(ChildRef.Index))), &(LocalBounds.Center.X));
		VectorStoreAligned(VectorMultiply(ZeroW, VectorSetFloat1(ChildExtent)), &(LocalBounds.Extent.X));
		return FOctreeNodeContext(LocalBounds, InInCullBits, InOutCullBits);
	}
	/**
	 * Determines which of the octree node's children intersect with a bounding box.
	 * @param BoundingBox - The bounding box to check for intersection with.
	 * @return A subset of the children's nodes that intersect the bounding box.
	 */
	[[nodiscard]] FOctreeChildNodeSubset GetIntersectingChildren(const FBoxCenterAndExtent& BoundingBox) const;

	/**
	 * Determines which of the octree node's children contain the whole bounding box, if any.
	 * @param BoundingBox - The bounding box to check for intersection with.
	 * @return The octree's node that the bounding box is farthest from the outside edge of, or an invalid node ref if it's not contained
	 *			by any of the children.
	 */
	[[nodiscard]] FOctreeChildNodeRef GetContainingChild(const FBoxCenterAndExtent& BoundingBox) const;
};

CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogGenericOctree, Log, All);

/** An octree. */
template<typename ElementType,typename OctreeSemantics>
class TOctree2
{
	using ElementArrayType = TArray<ElementType, typename OctreeSemantics::ElementAllocator>;
public:
	using FNodeIndex = uint32;
	using FReal = FVector::FReal;

private:
	struct FNode
	{
		FNodeIndex ChildNodes = INDEX_NONE;
		uint32 InclusiveNumElements = 0;

		bool IsLeaf() const
		{
			return ChildNodes == INDEX_NONE;
		}
	};

	FOctreeNodeContext RootNodeContext;
	TArray<FNode> TreeNodes;
	TArray<FNodeIndex> ParentLinks;
	TArray<ElementArrayType, TAlignedHeapAllocator<alignof(ElementArrayType)>> TreeElements;

	class FFreeList
	{
		struct FSpan
		{
			FNodeIndex Start;
			FNodeIndex End;
		};

		TArray<FSpan> FreeList;

	public:
		FFreeList()
		{
			Reset();
		}

		void Push(FNodeIndex NodeIndex)
		{
			//find the index that points to our right side node
			int Index = 1; //exclude the dummy
			int Size = FreeList.Num() - 1;

			//start with binary search for larger lists
			while (Size > 32)
			{
				const int LeftoverSize = Size % 2;
				Size = Size / 2;

				const int CheckIndex = Index + Size;
				const int IndexIfLess = CheckIndex + LeftoverSize;

				Index = FreeList[CheckIndex].Start > NodeIndex ? IndexIfLess : Index;
			}

			//small size array optimization
			int ArrayEnd = FMath::Min(Index + Size + 1, FreeList.Num());
			while (Index < ArrayEnd)
			{
				if (FreeList[Index].Start < NodeIndex)
				{
					break;
				}
				Index++;
			}

			//can we merge with the right node
			if (Index < FreeList.Num() && FreeList[Index].End + 1 == NodeIndex)
			{
				FreeList[Index].End = NodeIndex;

				//are we filling the gap between the left and right node
				if (FreeList[Index - 1].Start - 1 == NodeIndex)
				{
					FreeList[Index - 1].Start = FreeList[Index].Start;
					FreeList.RemoveAt(Index);
				}
				return;
			}

			//can we merge with the left node
			if (FreeList[Index - 1].Start - 1 == NodeIndex)
			{
				FreeList[Index - 1].Start = NodeIndex;
				return;
			}

			//we are node that could not be merged
			FreeList.Insert(FSpan{ NodeIndex , NodeIndex }, Index);
		}

		FNodeIndex Pop()
		{
			FSpan& Span = FreeList.Last();
			FNodeIndex Index = Span.Start;
			checkSlow(Index != INDEX_NONE);
			if (Span.Start == Span.End)
			{
				FreeList.Pop();
				return Index;
			}
			else
			{
				Span.Start++;
				return Index;
			}
		}

		void Reset()
		{
			FreeList.Reset(1);
			//push a dummy
			FreeList.Push(FSpan{ INDEX_NONE, INDEX_NONE });
		}

		[[nodiscard]] int Num() const
		{
			//includes one dummy
			return FreeList.Num() - 1;
		}
	};

	TArray<FNodeIndex> FreeList;
	/** The extent of a leaf at the maximum allowed depth of the tree. */
	FVector::FReal MinLeafExtent;

	FNodeIndex AllocateEightNodes()
	{
		FNodeIndex Index = INDEX_NONE;
		if (FreeList.Num())
		{
			Index = (FreeList.Pop() * 8) + 1;
		}
		else
		{
			Index = TreeNodes.AddDefaulted(8);
			ParentLinks.AddDefaulted();
			FNodeIndex ElementIndex = TreeElements.AddDefaulted(8);
			checkSlow(Index == ElementIndex);
		}
		return Index;
	}

	void FreeEightNodes(FNodeIndex Index)
	{
		checkSlow(Index != INDEX_NONE && Index != 0);
		for (int8 i = 0; i < 8; i++)
		{
			TreeNodes[Index + i] = FNode();
			checkSlow(TreeElements[Index + i].Num() == 0);
		}
		ParentLinks[(Index - 1) / 8] = INDEX_NONE;
		//TODO shrink the TreeNodes as well as the TreeElements and ParentLinks to reduce high watermark memory footprint.
		FreeList.Push((Index - 1) / 8);
	}

	void AddElementInternal(FNodeIndex CurrentNodeIndex, const FOctreeNodeContext& NodeContext, const FBoxCenterAndExtent& ElementBounds, typename TCallTraits<ElementType>::ConstReference Element, ElementArrayType& TempElementStorage)
	{
		checkSlow(CurrentNodeIndex != INDEX_NONE);
		TreeNodes[CurrentNodeIndex].InclusiveNumElements++;
		if (TreeNodes[CurrentNodeIndex].IsLeaf())
		{
			if (TreeElements[CurrentNodeIndex].Num() + 1 > OctreeSemantics::MaxElementsPerLeaf && NodeContext.Bounds.Extent.X > MinLeafExtent)
			{
				TempElementStorage = MoveTemp(TreeElements[CurrentNodeIndex]);

				FNodeIndex ChildStartIndex = AllocateEightNodes();
				ParentLinks[(ChildStartIndex - 1) / 8] = CurrentNodeIndex;
				TreeNodes[CurrentNodeIndex].ChildNodes = ChildStartIndex;
				TreeNodes[CurrentNodeIndex].InclusiveNumElements = 0;

				for (typename TCallTraits<ElementType>::ConstReference ChildElement : TempElementStorage)
				{
					const FBoxCenterAndExtent ChildElementBounds(OctreeSemantics::GetBoundingBox(ChildElement));
					AddElementInternal(CurrentNodeIndex, NodeContext, ChildElementBounds, ChildElement, TempElementStorage);
				}

				TempElementStorage.Reset();
				AddElementInternal(CurrentNodeIndex, NodeContext, ElementBounds, Element, TempElementStorage);
				return;
			}
			else
			{
				int ElementIndex = TreeElements[CurrentNodeIndex].Emplace(Element);

				SetElementId(Element, FOctreeElementId2(CurrentNodeIndex, ElementIndex));	
				return;
			}
		}
		else
		{
			const FOctreeChildNodeRef ChildRef = NodeContext.GetContainingChild(ElementBounds);
			if (ChildRef.IsNULL())
			{
				int ElementIndex = TreeElements[CurrentNodeIndex].Emplace(Element);
				SetElementId(Element, FOctreeElementId2(CurrentNodeIndex, ElementIndex));
				return;
			}
			else
			{
				FNodeIndex ChildNodeIndex = TreeNodes[CurrentNodeIndex].ChildNodes + ChildRef.Index;
				FOctreeNodeContext ChildNodeContext = NodeContext.GetChildContext(ChildRef);
				AddElementInternal(ChildNodeIndex, ChildNodeContext, ElementBounds, Element, TempElementStorage);
				return;
			}
		}
	}

	void CollapseNodesInternal(FNodeIndex CurrentNodeIndex, ElementArrayType& CollapsedNodeElements)
	{
		CollapsedNodeElements.Append(MoveTemp(TreeElements[CurrentNodeIndex]));
		TreeElements[CurrentNodeIndex].Reset();

		if (!TreeNodes[CurrentNodeIndex].IsLeaf())
		{
			FNodeIndex ChildStartIndex = TreeNodes[CurrentNodeIndex].ChildNodes;
			for (int8 i = 0; i < 8; i++)
			{
				CollapseNodesInternal(ChildStartIndex + i, CollapsedNodeElements);
			}

			// Mark the node as a leaf.
			TreeNodes[CurrentNodeIndex].ChildNodes = INDEX_NONE;

			FreeEightNodes(ChildStartIndex);
		}
	}

	template<typename PredicateFunc, typename IterateFunc>
	void FindNodesWithPredicateInternal(FNodeIndex ParentNodeIndex, FNodeIndex CurrentNodeIndex, const FOctreeNodeContext& NodeContext, const PredicateFunc& Predicate, const IterateFunc& Func) const
	{
		if (TreeNodes[CurrentNodeIndex].InclusiveNumElements > 0)
		{
			if (Predicate(ParentNodeIndex, CurrentNodeIndex, NodeContext.Bounds))
			{
				Func(ParentNodeIndex, CurrentNodeIndex, NodeContext.Bounds);

				if (!TreeNodes[CurrentNodeIndex].IsLeaf())
				{
					FNodeIndex ChildStartIndex = TreeNodes[CurrentNodeIndex].ChildNodes;
					for (int8 i = 0; i < 8; i++)
					{
						FindNodesWithPredicateInternal(CurrentNodeIndex, ChildStartIndex + i, NodeContext.GetChildContext(FOctreeChildNodeRef(i)), Predicate, Func);
					}
				}
			}
		}
	}

	template<typename IterateFunc>
	void FindElementsWithBoundsTestInternal(FNodeIndex CurrentNodeIndex, const FOctreeNodeContext& NodeContext, const FBoxCenterAndExtent& BoxBounds, const IterateFunc& Func) const
	{
		if (TreeNodes[CurrentNodeIndex].InclusiveNumElements > 0)
		{
			for (typename TCallTraits<ElementType>::ConstReference Element : TreeElements[CurrentNodeIndex])
			{
				if (Intersect(OctreeSemantics::GetBoundingBox(Element), BoxBounds))
				{
					Func(Element);
				}
			}

			if (!TreeNodes[CurrentNodeIndex].IsLeaf())
			{
				const FOctreeChildNodeSubset IntersectingChildSubset = NodeContext.GetIntersectingChildren(BoxBounds);
				FNodeIndex ChildStartIndex = TreeNodes[CurrentNodeIndex].ChildNodes;
				for (int8 i = 0; i < 8; i++)
				{
					if(IntersectingChildSubset.Contains(FOctreeChildNodeRef(i)))
					{
						FindElementsWithBoundsTestInternal(ChildStartIndex + i, NodeContext.GetChildContext(FOctreeChildNodeRef(i)), BoxBounds, Func);
					}
				}
			}
		}
	}

	template<typename IterateFunc>
	FOctreeElementId2 FindFirstElementWithBoundsTestInternal(FNodeIndex CurrentNodeIndex, const FOctreeNodeContext& NodeContext, const FBoxCenterAndExtent& BoxBounds, const IterateFunc& Func) const
	{
		if (TreeNodes[CurrentNodeIndex].InclusiveNumElements > 0)
		{
			for (int Index = 0; Index < TreeElements[CurrentNodeIndex].Num(); Index++)
			{
				typename TCallTraits<ElementType>::ConstReference Element = TreeElements[CurrentNodeIndex][Index];
				if (Intersect(OctreeSemantics::GetBoundingBox(Element), BoxBounds))
				{
					if (!Func(Element))
					{
						return FOctreeElementId2(CurrentNodeIndex, Index);
					}
				}
			}

			if (!TreeNodes[CurrentNodeIndex].IsLeaf())
			{
				const FOctreeChildNodeSubset IntersectingChildSubset = NodeContext.GetIntersectingChildren(BoxBounds);
				FNodeIndex ChildStartIndex = TreeNodes[CurrentNodeIndex].ChildNodes;
				for (int8 i = 0; i < 8; i++)
				{
					if (IntersectingChildSubset.Contains(FOctreeChildNodeRef(i)))
					{
						FOctreeElementId2 FoundIndex = FindFirstElementWithBoundsTestInternal(ChildStartIndex + i, NodeContext.GetChildContext(FOctreeChildNodeRef(i)), BoxBounds, Func);
						if (FoundIndex.IsValidId())
						{
							return FoundIndex;
						}
					}
				}
			}
		}

		// No matching element was found
		return FOctreeElementId2();
	}

	template<typename IterateFunc>
	void FindNearbyElementsInternal(FNodeIndex CurrentNodeIndex, const FOctreeNodeContext& NodeContext, const FBoxCenterAndExtent& BoxBounds, const IterateFunc& Func) const
	{
		if (TreeNodes[CurrentNodeIndex].InclusiveNumElements > 0)
		{
			for (int Index = 0; Index < TreeElements[CurrentNodeIndex].Num(); Index++)
			{
				Func(TreeElements[CurrentNodeIndex][Index]);
			}

			if (!TreeNodes[CurrentNodeIndex].IsLeaf())
			{
				// Find the child of the current node, if any, that contains the current new point
				FOctreeChildNodeRef ChildRef = NodeContext.GetContainingChild(BoxBounds);
				if (!ChildRef.IsNULL())
				{
					FNodeIndex ChildStartIndex = TreeNodes[CurrentNodeIndex].ChildNodes;
					// If the specified child node exists and contains any match, push it than process it
					if (TreeNodes[ChildStartIndex + ChildRef.Index].InclusiveNumElements > 0)
					{
						FindNearbyElementsInternal(ChildStartIndex + ChildRef.Index, NodeContext.GetChildContext(ChildRef), BoxBounds, Func);
					}
					// If the child node doesn't is a match, it's not worth pursuing any further. In an attempt to find
					// anything to match vs. the new point, process all of the children of the current octree node
					else
					{
						for (int8 i = 0; i < 8; i++)
						{
							FindNearbyElementsInternal(ChildStartIndex + i, NodeContext.GetChildContext(FOctreeChildNodeRef(i)), BoxBounds, Func);
						}
					}
				}
			}
		}
	}
public:

	[[nodiscard]] int32 GetNumNodes() const
	{
		return TreeNodes.Num();
	}

	/**
	 * this function will call the passed in function for all elements in the Octree in node by node in no specified order.
	 * @param Func - Function to call with each Element.
	 */
	template<typename IterateAllElementsFunc>
	inline void FindAllElements(const IterateAllElementsFunc& Func) const
	{
		for (const ElementArrayType& Elements : TreeElements)
		{
			for (typename TCallTraits<ElementType>::ConstReference Element : Elements)
			{
				Func(Element);
			}
		}
	}

	/**
	 * this function will traverse the Octree starting from the root in depth first order and the predicate can be used to implement custom culling for each node.
	 * @param Predicate - a Function when given the bounds of the currently traversed node that returns true if traversal should continue or false to skip that branch.
	 * @param Func - Function that will receive the node ID which can be stored and later used to get the elements using GetElementsForNode for all nodes that passed the predicate.
	 */
	template<typename PredicateFunc, typename IterateFunc>
	inline void FindNodesWithPredicate(const PredicateFunc& Predicate, const IterateFunc& Func) const
	{
		FindNodesWithPredicateInternal(INDEX_NONE, 0, RootNodeContext, Predicate, Func);
	}

	/**
	 * this function will traverse the Octree starting from the root in depth first order and the predicate can be used to implement custom culling for each node.
	 * @param Predicate - a Function when given the bounds of the currently traversed node that returns true if traversal should continue or false to skip that branch.
	 * @param Func - Function to call with each Element for nodes that passed the predicate.
	 */
	template<typename PredicateFunc, typename IterateFunc>
	inline void FindElementsWithPredicate(const PredicateFunc& Predicate, const IterateFunc& Func) const
	{
		FindNodesWithPredicateInternal(INDEX_NONE, 0, RootNodeContext, Predicate, [&Func, this](FNodeIndex /*ParentNodeIndex*/, FNodeIndex NodeIndex, const FBoxCenterAndExtent& /*NodeBounds*/ )
		{
			for (typename TCallTraits<ElementType>::ConstReference Element : TreeElements[NodeIndex])
			{
				Func(NodeIndex, Element);
			}
		});
	}

	/**
	 * this function will traverse the Octree using a fast box-box intersection this should be the preferred way of traversing the tree.
	 * @param BoxBounds - the bounds to test if a node is traversed or skipped.
	 * @param Func - Function to call with each Element for nodes that passed bounds test.
	 */
	template<typename IterateBoundsFunc>
	inline void FindElementsWithBoundsTest(const FBoxCenterAndExtent& BoxBounds, const IterateBoundsFunc& Func) const
	{
		FindElementsWithBoundsTestInternal(0, RootNodeContext, BoxBounds, Func);
	}

	/**
	 * this function will traverse the Octree using a fast box-box intersection and aborts traversal as soon as the Element function returns false.
	 * @param BoxBounds - the bounds to test if a node is traversed or skipped.
	 * @param Func - Function to call with each Element for nodes that passed bounds test.
	 * @return The ID of the found element. It's only valid until the next time the tree changes.
	 */
	template<typename IterateBoundsFunc>
	inline FOctreeElementId2 FindFirstElementWithBoundsTest(const FBoxCenterAndExtent& BoxBounds, const IterateBoundsFunc& Func) const
	{
		return FindFirstElementWithBoundsTestInternal(0, RootNodeContext, BoxBounds, Func);
	}

	/**
	* this function will traverse the Octree using a tryint to find nearby nodes that contain any elements.
	* @param Position - the position to look nearby.
	* @param Func - Function to call with each Element for nodes that passed bounds test.
	*/
	template<typename IterateBoundsFunc>
	inline void FindNearbyElements(const FVector& Position, const IterateBoundsFunc& Func) const
	{
		FindNearbyElementsInternal(0, RootNodeContext, FBoxCenterAndExtent(Position, FVector::ZeroVector), Func);
	}


	/**
	 * Adds an element to the octree.
	 * @param Element - The element to add.
	 */
	inline void AddElement(typename TCallTraits<ElementType>::ConstReference Element)
	{
		ElementArrayType TempElementStorage;
		const FBoxCenterAndExtent ElementBounds(OctreeSemantics::GetBoundingBox(Element));
		AddElementInternal(0, RootNodeContext, ElementBounds, Element, TempElementStorage);
	}

	/**
	 * Removes an element from the octree.
	 * @param ElementId - The element to remove from the octree.
	 */
	inline void RemoveElement(FOctreeElementId2 ElementId)
	{
		checkSlow(ElementId.IsValidId());

		// Remove the element from the node's element list.
		TreeElements[ElementId.NodeIndex].RemoveAtSwap(ElementId.ElementIndex, EAllowShrinking::No);

		if (ElementId.ElementIndex < TreeElements[ElementId.NodeIndex].Num())
		{
			// Update the external element id for the element that was swapped into the vacated element index.
			SetElementId(TreeElements[ElementId.NodeIndex][ElementId.ElementIndex], ElementId);
		}

		FNodeIndex CollapseNodeIndex = INDEX_NONE;
		{
			// Update the inclusive element counts for the nodes between the element and the root node,
			// and find the largest node that is small enough to collapse.
			FNodeIndex NodeIndex = ElementId.NodeIndex;
			while (true)
			{
				TreeNodes[NodeIndex].InclusiveNumElements--;
				if (TreeNodes[NodeIndex].InclusiveNumElements < OctreeSemantics::MinInclusiveElementsPerNode)
				{
					CollapseNodeIndex = NodeIndex;
				}

				if (NodeIndex == 0)
				{
					break;
				}

				NodeIndex = ParentLinks[(NodeIndex - 1) / 8];			
			}
		}

		// Collapse the largest node that was pushed below the threshold for collapse by the removal.
		if (CollapseNodeIndex != INDEX_NONE && !TreeNodes[CollapseNodeIndex].IsLeaf())
		{
			if (TreeElements[CollapseNodeIndex].Num() < (int32)TreeNodes[CollapseNodeIndex].InclusiveNumElements)
			{
				ElementArrayType TempElementStorage;
				TempElementStorage.Reserve(TreeNodes[CollapseNodeIndex].InclusiveNumElements);
				// Gather the elements contained in this node and its children.
				CollapseNodesInternal(CollapseNodeIndex, TempElementStorage);
				TreeElements[CollapseNodeIndex] = MoveTemp(TempElementStorage);

				for (int ElementIndex = 0; ElementIndex < TreeElements[CollapseNodeIndex].Num(); ElementIndex++)
				{
					// Update the external element id for the element that's being collapsed.
					SetElementId(TreeElements[CollapseNodeIndex][ElementIndex], FOctreeElementId2(CollapseNodeIndex, ElementIndex));
				}
			}
		}
	}

	/**
	 * this function resets the octree to empty.
	 */
	void Destroy()
	{
		TreeNodes.Reset(1);
		TreeElements.Reset(1);
		FreeList.Reset();
		ParentLinks.Reset();
		TreeNodes.AddDefaulted();
		TreeElements.AddDefaulted();
	}

	/** Accesses an octree element by ID. */
	[[nodiscard]] ElementType& GetElementById(FOctreeElementId2 ElementId)
	{
		return TreeElements[ElementId.NodeIndex][ElementId.ElementIndex];
	}

	/** Accesses an octree element by ID. */
	[[nodiscard]] const ElementType& GetElementById(FOctreeElementId2 ElementId) const
	{
		return TreeElements[ElementId.NodeIndex][ElementId.ElementIndex];
	}

	/**
	 * check if a FOctreeElementId2 is valid.
	 * @param ElementId - The ElementId to check.
	 */
	[[nodiscard]] bool IsValidElementId(FOctreeElementId2 ElementId) const
	{
		return ElementId.IsValidId() && ElementId.ElementIndex < TreeElements[ElementId.NodeIndex].Num();
	};

	/**
	 * return all elements for a given node.
	 * @param NodeIndex - The the index of the node can be obtained using FindNodesWithPredicate.
	 */
	[[nodiscard]] TArrayView<const ElementType> GetElementsForNode(FNodeIndex NodeIndex) const
	{
		return TreeElements[NodeIndex];
	}

	/** Writes stats for the octree to the log. */
	void DumpStats() const
	{
		int32 NumNodes = 0;
		int32 NumLeaves = 0;
		int32 NumElements = 0;
		int32 MaxElementsPerNode = 0;
		TArray<int32> NodeElementDistribution;

		FindNodesWithPredicateInternal(INDEX_NONE, 0, RootNodeContext, [](FNodeIndex /*ParentNodeIndex*/, FNodeIndex /*NodeIndex*/, const FBoxCenterAndExtent&) { return true; }, [&, this](FNodeIndex /*ParentNodeIndex*/, FNodeIndex NodeIndex, const FBoxCenterAndExtent&)
		{
			const int32 CurrentNodeElementCount = GetElementsForNode(NodeIndex).Num();

			NumNodes++;
			if (TreeNodes[NodeIndex].IsLeaf())
			{
				NumLeaves++;
			}

			NumElements += CurrentNodeElementCount;
			MaxElementsPerNode = FMath::Max(MaxElementsPerNode, CurrentNodeElementCount);

			if (CurrentNodeElementCount >= NodeElementDistribution.Num())
			{
				NodeElementDistribution.AddZeroed(CurrentNodeElementCount - NodeElementDistribution.Num() + 1);
			}
			NodeElementDistribution[CurrentNodeElementCount]++;
		});

		UE_LOGF(LogGenericOctree, Log, "Octree overview:");
		UE_LOGF(LogGenericOctree, Log, "\t%i nodes", NumNodes);
		UE_LOGF(LogGenericOctree, Log, "\t%i leaves", NumLeaves);
		UE_LOGF(LogGenericOctree, Log, "\t%i elements", NumElements);
		UE_LOGF(LogGenericOctree, Log, "\t%i >= elements per node", MaxElementsPerNode);
		UE_LOGF(LogGenericOctree, Log, "Octree node element distribution:");
		for (int32 i = 0; i < NodeElementDistribution.Num(); i++)
		{
			if (NodeElementDistribution[i] > 0)
			{
				UE_LOGF(LogGenericOctree, Log, "\tElements: %3i, Nodes: %3i", i, NodeElementDistribution[i]);
			}
		}
	}

	[[nodiscard]] SIZE_T GetSizeBytes() const
	{
		SIZE_T TotalSizeBytes = TreeNodes.GetAllocatedSize();
		TotalSizeBytes += TreeElements.GetAllocatedSize();
		TotalSizeBytes += TreeNodes[0].InclusiveNumElements * sizeof(ElementType);
		return TotalSizeBytes;
	}

	[[nodiscard]] FReal GetNodeLevelExtent(int32 Level) const
	{
		const int32 ClampedLevel = FMath::Clamp<uint32>(Level, 0, OctreeSemantics::MaxNodeDepth);
		return RootNodeContext.Bounds.Extent.X * FMath::Pow((1.0f + 1.0f / (FReal)FOctreeNodeContext::LoosenessDenominator) / 2.0f, FReal(ClampedLevel));
	}

	[[nodiscard]] FBoxCenterAndExtent GetRootBounds() const
	{
		return RootNodeContext.Bounds;
	}

	void ShrinkElements()
	{
		for (ElementArrayType& Elements : TreeElements)
		{
			Elements.Shrink();
		}
	}

	/** 
	 * Apply an arbitrary offset to all elements in the tree 
	 * InOffset - offset to apply
	 * bGlobalOctree - hint that this octree is used as a boundless global volume, 
	 *  so only content will be shifted but not origin of the octree
	 */
	void ApplyOffset(const FVector& InOffset, bool bGlobalOctree = false)
	{
		ElementArrayType TempElementStorage;
		TempElementStorage.Reserve(TreeNodes[0].InclusiveNumElements);

		//collect all elements
		CollapseNodesInternal(0, TempElementStorage);
		checkSlow(TreeNodes[0].IsLeaf());
		Destroy();

		if (!bGlobalOctree)
		{
			RootNodeContext.Bounds.Center += FVector4(InOffset, 0.0f);
		}

		// Offset & Add all elements from a saved nodes to a new empty octree
		for (ElementType& Element : TempElementStorage)
		{
			OctreeSemantics::ApplyOffset(Element, InOffset);
			AddElement(Element);
		}
	}

	/** Initialization constructor. */
	TOctree2(const FVector& InOrigin,FVector::FReal InExtent)
		: RootNodeContext(FBoxCenterAndExtent(InOrigin, FVector(InExtent, InExtent, InExtent)), 0, 0)
		, MinLeafExtent(InExtent* FMath::Pow((1.0f + 1.0f / (FReal)FOctreeNodeContext::LoosenessDenominator) / 2.0f, FReal(OctreeSemantics::MaxNodeDepth)))
	{
		TreeNodes.AddDefaulted();
		TreeElements.AddDefaulted();
	}

	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */
	TOctree2() 
	{
		TreeNodes.AddDefaulted();
		TreeElements.AddDefaulted();
	}

private:


	// Concept definition for the new octree semantics, which adds a new TOctree parameter
	struct COctreeSemanticsV2
	{
		template<typename Semantics>
		auto Requires(typename Semantics::FOctree& OctreeInstance, const ElementType& Element, FOctreeElementId2 Id)
			-> decltype(Semantics::SetElementId(OctreeInstance, Element, Id));
	};

	// Function overload set which calls the V2 version if it's supported or the old version if it's not
	template <typename Semantics>
	void SetOctreeSemanticsElementId(const ElementType& Element, FOctreeElementId2 Id)
	{
		if constexpr (TModels_V<COctreeSemanticsV2, Semantics>)
		{
			Semantics::SetElementId(static_cast<typename Semantics::FOctree&>(*this), Element, Id);
		}
		else
		{
			Semantics::SetElementId(Element, Id);
		}
	}

protected:
	// redirects SetElementId call to the proper implementation
	void SetElementId(const ElementType& Element, FOctreeElementId2 Id)
	{
		SetOctreeSemanticsElementId<OctreeSemantics>(Element, Id);
	}
};

/** An octree. */
template<typename ElementType, typename OctreeSemantics>
class TOctree_DEPRECATED
{
public:

	using FReal = FVector4::FReal;
	typedef TArray<ElementType, typename OctreeSemantics::ElementAllocator> ElementArrayType;
	typedef typename ElementArrayType::TConstIterator ElementConstIt;

	/** A node in the octree. */
	class FNode
	{
	public:

		friend class TOctree_DEPRECATED;

		/** Initialization constructor. */
		explicit FNode(const FNode* InParent)
			: Parent(InParent)
			, InclusiveNumElements(0)
			, bIsLeaf(true)
		{
			FOREACH_OCTREE_CHILD_NODE(ChildRef)
			{
				Children[ChildRef.Index] = NULL;
			}
		}

		/** Destructor. */
		~FNode()
		{
			FOREACH_OCTREE_CHILD_NODE(ChildRef)
			{
				delete Children[ChildRef.Index];
			}
		}

		// Accessors.
		[[nodiscard]] UE_FORCEINLINE_HINT ElementConstIt GetElementIt() const
		{
			return ElementConstIt(Elements);
		}
		[[nodiscard]] UE_FORCEINLINE_HINT bool IsLeaf() const
		{
			return bIsLeaf;
		}
		[[nodiscard]] UE_FORCEINLINE_HINT bool HasChild(FOctreeChildNodeRef ChildRef) const
		{
			return Children[ChildRef.Index] != NULL && Children[ChildRef.Index]->InclusiveNumElements > 0;
		}
		[[nodiscard]] UE_FORCEINLINE_HINT FNode* GetChild(FOctreeChildNodeRef ChildRef) const
		{
			return Children[ChildRef.Index];
		}
		[[nodiscard]] UE_FORCEINLINE_HINT int32 GetElementCount() const
		{
			return Elements.Num();
		}
		[[nodiscard]] UE_FORCEINLINE_HINT int32 GetInclusiveElementCount() const
		{
			return InclusiveNumElements;
		}
		[[nodiscard]] UE_FORCEINLINE_HINT const ElementArrayType& GetElements() const
		{
			return Elements;
		}
		void ShrinkElements()
		{
			Elements.Shrink();
			FOREACH_OCTREE_CHILD_NODE(ChildRef)
			{
				if (Children[ChildRef.Index])
				{
					Children[ChildRef.Index]->ShrinkElements();
				}
			}
		}

		void ApplyOffset(const FVector& InOffset)
		{
			for (auto& Element : Elements)
			{
				OctreeSemantics::ApplyOffset(Element, InOffset);
			}

			FOREACH_OCTREE_CHILD_NODE(ChildRef)
			{
				if (Children[ChildRef.Index])
				{
					Children[ChildRef.Index]->ApplyOffset(InOffset);
				}
			}
		}

	private:

		/** The elements in this node. */
		mutable ElementArrayType Elements;

		/** The parent of this node. */
		const FNode* Parent;

		/** The children of the node. */
		mutable FNode* Children[8];

		/** The number of elements contained by the node and its child nodes. */
		mutable uint32 InclusiveNumElements : 31;

		/** true if the meshes should be added directly to the node, rather than subdividing when possible. */
		mutable uint32 bIsLeaf : 1;
	};



	/** A reference to an octree node, its context, and a read lock. */
	class FNodeReference
	{
	public:

		const FNode* Node;
		FOctreeNodeContext Context;

		/** Default constructor. */
		FNodeReference() :
			Node(NULL),
			Context()
		{}

		/** Initialization constructor. */
		FNodeReference(const FNode* InNode, const FOctreeNodeContext& InContext) :
			Node(InNode),
			Context(InContext)
		{}
	};

	/** The default iterator allocator gives the stack enough inline space to contain a path and its siblings from root to leaf. */
	typedef TInlineAllocator<7 * (14 - 1) + 8> DefaultStackAllocator;

	/** An octree node iterator. */
	template<typename StackAllocator = DefaultStackAllocator>
	class TConstIterator
	{
	public:

		/** Pushes a child of the current node onto the stack of nodes to visit. */
		void PushChild(FOctreeChildNodeRef ChildRef)
		{
			FNodeReference* NewNode = new (NodeStack) FNodeReference;
			NewNode->Node = CurrentNode.Node->GetChild(ChildRef);
			CurrentNode.Context.GetChildContext(ChildRef, &NewNode->Context);
		}
		/** Pushes a child of the current node onto the stack of nodes to visit. */
		void PushChild(FOctreeChildNodeRef ChildRef, uint32 FullyInsideView, uint32 FullyOutsideView)
		{
			FNodeReference* NewNode = new (NodeStack) FNodeReference;
			NewNode->Node = CurrentNode.Node->GetChild(ChildRef);
			CurrentNode.Context.GetChildContext(ChildRef, &NewNode->Context);
			NewNode->Context.InCullBits = FullyInsideView;
			NewNode->Context.OutCullBits = FullyOutsideView;
		}
		/** Pushes a child of the current node onto the stack of nodes to visit. */
		void PushChild(FOctreeChildNodeRef ChildRef, const FOctreeNodeContext& Context)
		{
			new (NodeStack) FNodeReference(CurrentNode.Node->GetChild(ChildRef), Context);
		}


		/** Iterates to the next node. */
		void Advance()
		{
			if (NodeStack.Num())
			{
				CurrentNode = NodeStack[NodeStack.Num() - 1];
				NodeStack.RemoveAt(NodeStack.Num() - 1);
			}
			else
			{
				CurrentNode = FNodeReference();
			}
		}

		/** Checks if there are any nodes left to iterate over. */
		[[nodiscard]] bool HasPendingNodes() const
		{
			return CurrentNode.Node != NULL;
		}

		/** Starts iterating at the root of an octree. */
		TConstIterator(const TOctree_DEPRECATED& Tree)
			: CurrentNode(FNodeReference(&Tree.RootNode, Tree.RootNodeContext))
		{}

		/** Starts iterating at a particular node of an octree. */
		TConstIterator(const FNode& Node, const FOctreeNodeContext& Context) :
			CurrentNode(FNodeReference(&Node, Context))
		{}

		// Accessors.
		[[nodiscard]] const FNode& GetCurrentNode() const
		{
			return *CurrentNode.Node;
		}
		[[nodiscard]] const FOctreeNodeContext& GetCurrentContext() const
		{
			return CurrentNode.Context;
		}

	private:

		/** The node that is currently being visited. */
		FNodeReference CurrentNode;

		/** The nodes which are pending iteration. */
		TArray<FNodeReference, StackAllocator> NodeStack;
	};

	/** Iterates over the elements in the octree that intersect a bounding box. */
	template<typename StackAllocator = DefaultStackAllocator>
	class TConstElementBoxIterator
	{
	public:

		/** Iterates to the next element. */
		void Advance()
		{
			++ElementIt;
			AdvanceToNextIntersectingElement();
		}

		/** Checks if there are any elements left to iterate over. */
		[[nodiscard]] bool HasPendingElements() const
		{
			return NodeIt.HasPendingNodes();
		}

		/** Initialization constructor. */
		TConstElementBoxIterator(const TOctree_DEPRECATED& Tree, const FBoxCenterAndExtent& InBoundingBox)
			: IteratorBounds(InBoundingBox)
			, NodeIt(Tree)
			, ElementIt(Tree.RootNode.GetElementIt())
		{
			ProcessChildren();
			AdvanceToNextIntersectingElement();
		}

		// Accessors.
		[[nodiscard]] const ElementType& GetCurrentElement() const
		{
			return *ElementIt;
		}

	private:

		/** The bounding box to check for intersection with. */
		FBoxCenterAndExtent IteratorBounds;

		/** The octree node iterator. */
		TConstIterator<StackAllocator> NodeIt;

		/** The element iterator for the current node. */
		ElementConstIt ElementIt;

		/** Processes the children of the current node. */
		void ProcessChildren()
		{
			// Add the child nodes that intersect the bounding box to the node iterator's stack.
			const FNode& CurrentNode = NodeIt.GetCurrentNode();
			const FOctreeNodeContext& Context = NodeIt.GetCurrentContext();
			const FOctreeChildNodeSubset IntersectingChildSubset = Context.GetIntersectingChildren(IteratorBounds);
			FOREACH_OCTREE_CHILD_NODE(ChildRef)
			{
				if (IntersectingChildSubset.Contains(ChildRef) && CurrentNode.HasChild(ChildRef))
				{
					NodeIt.PushChild(ChildRef);
				}
			}
		}

		/** Advances the iterator to the next intersecting primitive, starting at a primitive in the current node. */
		void AdvanceToNextIntersectingElement()
		{
			check(NodeIt.HasPendingNodes()); // please don't call this after iteration has ended

			while (1)
			{
				ElementConstIt LocalElementIt(ElementIt);
				if (LocalElementIt)
				{
					FPlatformMisc::Prefetch(&(*LocalElementIt));
					FPlatformMisc::Prefetch(&(*LocalElementIt), PLATFORM_CACHE_LINE_SIZE);

					// this is redundantly pull out of the while loop to prevent LHS on the iterator
					// Check if the current element intersects the bounding box.
					if (Intersect(OctreeSemantics::GetBoundingBox(*LocalElementIt), IteratorBounds))
					{
						// If it intersects, break out of the advancement loop.
						Move(ElementIt, LocalElementIt);
						return;
					}

					// Check if we've advanced past the elements in the current node.
					while (++LocalElementIt)
					{
						FPlatformMisc::Prefetch(&(*LocalElementIt), PLATFORM_CACHE_LINE_SIZE);

						// Check if the current element intersects the bounding box.
						if (Intersect(OctreeSemantics::GetBoundingBox(*LocalElementIt), IteratorBounds))

						{
							// If it intersects, break out of the advancement loop.
							Move(ElementIt, LocalElementIt);
							return;
						}
					}
				}
				// Advance to the next node.
				NodeIt.Advance();
				if (!NodeIt.HasPendingNodes())
				{
					Move(ElementIt, LocalElementIt);
					return;
				}
				ProcessChildren();
				// The element iterator can't be assigned to, but it can be replaced by Move.
				Move(ElementIt, NodeIt.GetCurrentNode().GetElementIt());
			}
		}
	};

	/**
	 * Adds an element to the octree.
	 * @param Element - The element to add.
	 * @return An identifier for the element in the octree.
	 */
	void AddElement(typename TTypeTraits<ElementType>::ConstInitType Element)
	{
		AddElementToNode(Element, RootNode, RootNodeContext);
	}

	/**
	 * Removes an element from the octree.
	 * @param ElementId - The element to remove from the octree.
	 */
	void RemoveElement(FOctreeElementId ElementId)
	{
		check(ElementId.IsValidId());

		FNode* ElementIdNode = (FNode*)ElementId.Node;

		// Remove the element from the node's element list.
		ElementIdNode->Elements.RemoveAtSwap(ElementId.ElementIndex);

		SetOctreeMemoryUsage(this, TotalSizeBytes - sizeof(ElementType));

		if (ElementId.ElementIndex < ElementIdNode->Elements.Num())
		{
			// Update the external element id for the element that was swapped into the vacated element index.
			SetElementId(ElementIdNode->Elements[ElementId.ElementIndex], ElementId);
		}

		// Update the inclusive element counts for the nodes between the element and the root node,
		// and find the largest node that is small enough to collapse.
		const FNode* CollapseNode = NULL;
		for (const FNode* Node = ElementIdNode; Node; Node = Node->Parent)
		{
			--Node->InclusiveNumElements;
			if (Node->InclusiveNumElements < OctreeSemantics::MinInclusiveElementsPerNode)
			{
				CollapseNode = Node;
			}
		}

		// Collapse the largest node that was pushed below the threshold for collapse by the removal.
		if (CollapseNode && !CollapseNode->bIsLeaf)
		{
			if (CollapseNode->Elements.Num() < (int32)CollapseNode->InclusiveNumElements)
			{
				CollapseNode->Elements.Reserve(CollapseNode->InclusiveNumElements);

				// Gather the elements contained in this node and its children.
				for (TConstIterator<> ChildNodeIt(*CollapseNode, RootNodeContext); ChildNodeIt.HasPendingNodes(); ChildNodeIt.Advance())
				{
					const FNode& ChildNode = ChildNodeIt.GetCurrentNode();

					if (&ChildNode != CollapseNode)
					{
						// Child node will be collapsed so move the child's elements to the collapse node element list.
						for (ElementType& Element : ChildNode.Elements)
						{
							const int32 NewElementIndex = CollapseNode->Elements.Add(MoveTemp(Element));

							// Update the external element id for the element that's being collapsed.
							SetElementId(CollapseNode->Elements[NewElementIndex], FOctreeElementId(CollapseNode, NewElementIndex));
						}
					}

					// Recursively visit all child nodes.
					FOREACH_OCTREE_CHILD_NODE(ChildRef)
					{
						if (ChildNode.HasChild(ChildRef))
						{
							ChildNodeIt.PushChild(ChildRef);
						}
					}
				}

				// Free the child nodes.
				FOREACH_OCTREE_CHILD_NODE(ChildRef)
				{
					if (CollapseNode->Children[ChildRef.Index])
					{
						SetOctreeMemoryUsage(this, TotalSizeBytes - sizeof(*CollapseNode->Children[ChildRef.Index]));
					}

					delete CollapseNode->Children[ChildRef.Index];
					CollapseNode->Children[ChildRef.Index] = NULL;
				}
			}

			// Mark the node as a leaf.
			CollapseNode->bIsLeaf = true;
		}
	}


	void Destroy()
	{
		RootNode.~FNode();
		new (&RootNode) FNode(NULL);

		// this looks a bit @hacky, but FNode's destructor doesn't 
		// update TotalSizeBytes so better to set it to 0 than
		// not do nothing with it (would contain obviously false value)
		SetOctreeMemoryUsage(this, 0);
	}

	/** Accesses an octree element by ID. */
	[[nodiscard]] ElementType& GetElementById(FOctreeElementId ElementId)
	{
		check(ElementId.IsValidId());
		FNode* ElementIdNode = (FNode*)ElementId.Node;
		return ElementIdNode->Elements[ElementId.ElementIndex];
	}

	/** Accesses an octree element by ID. */
	[[nodiscard]] const ElementType& GetElementById(FOctreeElementId ElementId) const
	{
		check(ElementId.IsValidId());
		FNode* ElementIdNode = (FNode*)ElementId.Node;
		return ElementIdNode->Elements[ElementId.ElementIndex];
	}

	/** Checks if given ElementId represents a valid Octree element */
	[[nodiscard]] bool IsValidElementId(FOctreeElementId ElementId) const
	{
		return ElementId.IsValidId()
			&& ElementId.ElementIndex != INDEX_NONE
			&& ElementId.ElementIndex < ((FNode*)ElementId.Node)->Elements.Num();
	};

	/** Writes stats for the octree to the log. */
	void DumpStats() const
	{
		int32 NumNodes = 0;
		int32 NumLeaves = 0;
		int32 NumElements = 0;
		int32 MaxElementsPerNode = 0;
		TArray<int32> NodeElementDistribution;

		for (TConstIterator<> NodeIt(*this); NodeIt.HasPendingNodes(); NodeIt.Advance())
		{
			const FNode& CurrentNode = NodeIt.GetCurrentNode();
			const int32 CurrentNodeElementCount = CurrentNode.GetElementCount();

			NumNodes++;
			if (CurrentNode.IsLeaf())
			{
				NumLeaves++;
			}

			NumElements += CurrentNodeElementCount;
			MaxElementsPerNode = FMath::Max(MaxElementsPerNode, CurrentNodeElementCount);

			if (CurrentNodeElementCount >= NodeElementDistribution.Num())
			{
				NodeElementDistribution.AddZeroed(CurrentNodeElementCount - NodeElementDistribution.Num() + 1);
			}
			NodeElementDistribution[CurrentNodeElementCount]++;

			FOREACH_OCTREE_CHILD_NODE(ChildRef)
			{
				if (CurrentNode.HasChild(ChildRef))
				{
					NodeIt.PushChild(ChildRef);
				}
			}
		}

		UE_LOGF(LogGenericOctree, Log, "Octree overview:");
		UE_LOGF(LogGenericOctree, Log, "\t%i nodes", NumNodes);
		UE_LOGF(LogGenericOctree, Log, "\t%i leaves", NumLeaves);
		UE_LOGF(LogGenericOctree, Log, "\t%i elements", NumElements);
		UE_LOGF(LogGenericOctree, Log, "\t%i >= elements per node", MaxElementsPerNode);
		UE_LOGF(LogGenericOctree, Log, "Octree node element distribution:");
		for (int32 i = 0; i < NodeElementDistribution.Num(); i++)
		{
			if (NodeElementDistribution[i] > 0)
			{
				UE_LOGF(LogGenericOctree, Log, "\tElements: %3i, Nodes: %3i", i, NodeElementDistribution[i]);
			}
		}
	}


	[[nodiscard]] SIZE_T GetSizeBytes() const
	{
		return TotalSizeBytes;
	}

	[[nodiscard]] FReal GetNodeLevelExtent(int32 Level) const
	{
		const int32 ClampedLevel = FMath::Clamp<uint32>(Level, 0, OctreeSemantics::MaxNodeDepth);
		return RootNodeContext.Bounds.Extent.X * FMath::Pow((1.0f + 1.0f / (FReal)FOctreeNodeContext::LoosenessDenominator) / 2.0f, FReal(ClampedLevel));
	}

	[[nodiscard]] FBoxCenterAndExtent GetRootBounds() const
	{
		return RootNodeContext.Bounds;
	}

	void ShrinkElements()
	{
		RootNode.ShrinkElements();
	}

	/**
	 * Apply an arbitrary offset to all elements in the tree
	 * InOffset - offset to apply
	 * bGlobalOctree - hint that this octree is used as a boundless global volume,
	 *  so only content will be shifted but not origin of the octree
	 */
	void ApplyOffset(const FVector& InOffset, bool bGlobalOctree)
	{
		// Shift elements
		RootNode.ApplyOffset(InOffset);

		// Make a local copy of all nodes
		FNode OldRootNode = RootNode;

		// Assign to root node a NULL node, so Destroy procedure will not delete child nodes
		RootNode = FNode(nullptr);
		// Call destroy to clean up octree
		Destroy();

		if (!bGlobalOctree)
		{
			RootNodeContext.Bounds.Center += FVector4(InOffset, 0.0f);
		}

		// Add all elements from a saved nodes to a new empty octree
		for (TConstIterator<> NodeIt(OldRootNode, RootNodeContext); NodeIt.HasPendingNodes(); NodeIt.Advance())
		{
			const auto& CurrentNode = NodeIt.GetCurrentNode();

			FOREACH_OCTREE_CHILD_NODE(ChildRef)
			{
				if (CurrentNode.HasChild(ChildRef))
				{
					NodeIt.PushChild(ChildRef);
				}
			}

			for (auto ElementIt = CurrentNode.GetElementIt(); ElementIt; ++ElementIt)
			{
				AddElement(*ElementIt);
			}
		}

		// Saved nodes will be deleted on scope exit
	}


	/** Initialization constructor. */
	TOctree_DEPRECATED(const FVector& InOrigin, FReal InExtent)
		: RootNode(NULL)
		, RootNodeContext(FBoxCenterAndExtent(InOrigin, FVector(InExtent, InExtent, InExtent)), 0, 0)
		, MinLeafExtent(InExtent* FMath::Pow((1.0f + 1.0f / (FReal)FOctreeNodeContext::LoosenessDenominator) / 2.0f, FReal(OctreeSemantics::MaxNodeDepth)))
		, TotalSizeBytes(0)
	{
	}

	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */
	TOctree_DEPRECATED() : RootNode(nullptr)
	{
		EnsureRetrievingVTablePtrDuringCtor(TEXT("TOctree()"));
	}

private:

	/** The octree's root node. */
	FNode RootNode;

	/** The octree's root node's context. */
	FOctreeNodeContext RootNodeContext;

	/** The extent of a leaf at the maximum allowed depth of the tree. */
	FReal MinLeafExtent;

	/** this function basically set TotalSizeBytes, but gives opportunity to
	 *	include this Octree in memory stats */
	void SetOctreeMemoryUsage(TOctree_DEPRECATED<ElementType, OctreeSemantics>* Octree, int32 NewSize)
	{
		Octree->TotalSizeBytes = NewSize;
	}

	SIZE_T TotalSizeBytes;

	/** Adds an element to a node or its children. */
	void AddElementToNode(
		typename TTypeTraits<ElementType>::ConstInitType Element,
		const FNode& InNode,
		const FOctreeNodeContext& InContext
	)
	{
		const FBoxCenterAndExtent ElementBounds(OctreeSemantics::GetBoundingBox(Element));

		for (TConstIterator<TInlineAllocator<1> > NodeIt(InNode, InContext); NodeIt.HasPendingNodes(); NodeIt.Advance())
		{
			const FNode& Node = NodeIt.GetCurrentNode();
			const FOctreeNodeContext& Context = NodeIt.GetCurrentContext();
			const bool bIsLeaf = Node.IsLeaf();

			bool bAddElementToThisNode = false;

			// Increment the number of elements included in this node and its children.
			Node.InclusiveNumElements++;

			if (bIsLeaf)
			{
				// If this is a leaf, check if adding this element would turn it into a node by overflowing its element list.
				if (Node.Elements.Num() + 1 > OctreeSemantics::MaxElementsPerLeaf && Context.Bounds.Extent.X > MinLeafExtent)
				{
					// Copy the leaf's elements, remove them from the leaf, and turn it into a node.
					ElementArrayType ChildElements;
					Exchange(ChildElements, Node.Elements);
					SetOctreeMemoryUsage(this, TotalSizeBytes - ChildElements.Num() * sizeof(ElementType));
					Node.InclusiveNumElements = 0;

					// Allow elements to be added to children of this node.
					Node.bIsLeaf = false;

					// Re-add all of the node's child elements, potentially creating children of this node for them.
					for (ElementConstIt ElementIt(ChildElements); ElementIt; ++ElementIt)
					{
						AddElementToNode(*ElementIt, Node, Context);
					}

					// Add the element to this node.
					AddElementToNode(Element, Node, Context);
					return;
				}
				else
				{
					// If the leaf has room for the new element, simply add it to the list.
					bAddElementToThisNode = true;
				}
			}
			else
			{
				// If this isn't a leaf, find a child that entirely contains the element.
				const FOctreeChildNodeRef ChildRef = Context.GetContainingChild(ElementBounds);
				if (ChildRef.IsNULL())
				{
					// If none of the children completely contain the element, add it to this node directly.
					bAddElementToThisNode = true;
				}
				else
				{
					// Create the child node if it hasn't been created yet.
					if (!Node.Children[ChildRef.Index])
					{
						Node.Children[ChildRef.Index] = new typename TOctree_DEPRECATED<ElementType, OctreeSemantics>::FNode(&Node);
						SetOctreeMemoryUsage(this, TotalSizeBytes + sizeof(*Node.Children[ChildRef.Index]));
					}

					// Push the node onto the stack to visit.
					NodeIt.PushChild(ChildRef);
				}
			}

			if (bAddElementToThisNode)
			{
				// Add the element to this node.
				new(Node.Elements) ElementType(Element);

				SetOctreeMemoryUsage(this, TotalSizeBytes + sizeof(ElementType));

				// Set the element's ID.
				SetElementId(Element, FOctreeElementId(&Node, Node.Elements.Num() - 1));
				return;
			}
		}

		UE_LOGF(LogGenericOctree, Fatal,
			"Failed to find an octree node for an element with bounds (%f,%f,%f) +/- (%f,%f,%f)!",
			ElementBounds.Center.X,
			ElementBounds.Center.Y,
			ElementBounds.Center.Z,
			ElementBounds.Extent.X,
			ElementBounds.Extent.Y,
			ElementBounds.Extent.Z
		);
	}

	// Concept definition for the new octree semantics, which adds a new TOctree parameter
	struct COctreeSemanticsV2
	{
		template<typename Semantics>
		auto Requires(typename Semantics::FOctree& OctreeInstance, const ElementType& Element, FOctreeElementId Id)
			-> decltype(Semantics::SetElementId(OctreeInstance, Element, Id));
	};

	// Calls the V2 version if it's supported or the old version if it's not
	template <typename Semantics>
	void SetOctreeSemanticsElementId(const ElementType& Element, FOctreeElementId Id)
	{
		if constexpr (TModels_V<COctreeSemanticsV2, Semantics>)
		{
			Semantics::SetElementId(*this, Element, Id);
		}
		else
		{
			Semantics::SetElementId(Element, Id);
		}
	}

protected:
	// redirects SetElementId call to the proper implementation
	void SetElementId(const ElementType& Element, FOctreeElementId Id)
	{
		SetOctreeSemanticsElementId<OctreeSemantics>(Element, Id);
	}
};

template<typename ElementType, typename OctreeSemantics>
class UE_DEPRECATED(4.26, "The old Octree is deprecated use TOctree2.") TOctree : public TOctree_DEPRECATED<ElementType, OctreeSemantics>
{
public:
	TOctree(const FVector& InOrigin, FVector::FReal InExtent) : TOctree_DEPRECATED<ElementType, OctreeSemantics>(InOrigin, InExtent)
	{
	}

	TOctree() : TOctree_DEPRECATED<ElementType, OctreeSemantics>()
	{
	}
};

#include "Math/GenericOctree.inl" // IWYU pragma: export

#include <stddef.h> // IWYU pragma: export
