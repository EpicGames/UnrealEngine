// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GenericOctree.inl: Generic octree implementation.
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "CoreFwd.h"
#include "Logging/LogMacros.h"

class FBoxCenterAndExtent;
class FOctreeChildNodeRef;
class FOctreeChildNodeSubset;
class FOctreeNodeContext;
struct FMath;


inline bool FOctreeChildNodeSubset::Contains(FOctreeChildNodeRef ChildRef) const
{
	// This subset contains the child if it has all the bits set that are set for the subset containing only the child node.
	const FOctreeChildNodeSubset ChildSubset(ChildRef);
	return (ChildBits & ChildSubset.ChildBits) == ChildSubset.ChildBits;
}

inline FOctreeChildNodeSubset FOctreeNodeContext::GetIntersectingChildren(const FBoxCenterAndExtent& QueryBounds) const
{
	FOctreeChildNodeSubset Result;

	// Load the query bounding box values as VectorRegisters.
	const VectorRegister QueryBoundsCenter = VectorLoadAligned(&QueryBounds.Center);
	const VectorRegister QueryBoundsExtent = VectorLoadAligned(&QueryBounds.Extent);
	const VectorRegister QueryBoundsMax = VectorAdd(QueryBoundsCenter,QueryBoundsExtent);
	const VectorRegister QueryBoundsMin = VectorSubtract(QueryBoundsCenter,QueryBoundsExtent);

	// Compute the bounds of the node's children.
	const VectorRegister BoundsCenter = VectorLoadAligned(&Bounds.Center);
	const VectorRegister BoundsExtent = VectorLoadAligned(&Bounds.Extent);
	const VectorRegister PositiveChildBoundsMin = VectorSubtract(
		VectorAdd(BoundsCenter,VectorLoadFloat1(&ChildCenterOffset)),
		VectorLoadFloat1(&ChildExtent)
		);
	const VectorRegister NegativeChildBoundsMax = VectorAdd(
		VectorSubtract(BoundsCenter,VectorLoadFloat1(&ChildCenterOffset)),
		VectorLoadFloat1(&ChildExtent)
		);

	// Intersect the query bounds with the node's children's bounds.
	Result.PositiveChildBits = VectorMaskBits(VectorCompareGT(QueryBoundsMax, PositiveChildBoundsMin)) & 0x7;
	Result.NegativeChildBits = VectorMaskBits(VectorCompareLE(QueryBoundsMin, NegativeChildBoundsMax)) & 0x7;
	return Result;
}

inline FOctreeChildNodeRef FOctreeNodeContext::GetContainingChild(const FBoxCenterAndExtent& QueryBounds) const
{
	FOctreeChildNodeRef Result;

	// Load the query bounding box values as VectorRegisters.
	const VectorRegister QueryBoundsCenter = VectorLoadAligned(&QueryBounds.Center);
	const VectorRegister QueryBoundsExtent = VectorLoadAligned(&QueryBounds.Extent);

	// Compute the bounds of the node's children.
	const VectorRegister BoundsCenter = VectorLoadAligned(&Bounds.Center);
	const VectorRegister ChildCenterOffsetVector = VectorLoadFloat1(&ChildCenterOffset);
	const VectorRegister NegativeCenterDifference = VectorSubtract(QueryBoundsCenter,VectorSubtract(BoundsCenter,ChildCenterOffsetVector));
	const VectorRegister PositiveCenterDifference = VectorSubtract(VectorAdd(BoundsCenter,ChildCenterOffsetVector),QueryBoundsCenter);

	// If the query bounds isn't entirely inside the bounding box of the child it's closest to, it's not contained by any of the child nodes.
	const VectorRegister MinDifference = VectorMin(PositiveCenterDifference,NegativeCenterDifference);
	if(VectorAnyGreaterThan(VectorAdd(QueryBoundsExtent,MinDifference),VectorLoadFloat1(&ChildExtent)))
	{
		Result.SetNULL();
	}
	else
	{
		// Return the child node that the query is closest to as the containing child.
		Result.Index = VectorMaskBits(VectorCompareGT(QueryBoundsCenter, BoundsCenter)) & 0x7;
	}

	return Result;
}

