// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GenericOctreePublic.h: Generic octree definition.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

/** 
 *	An identifier for an element in the octree. 
 */
class FOctreeElementId2
{
public:

	template<typename, typename>
	friend class TOctree2;

	/** Default constructor. */
	[[nodiscard]] FOctreeElementId2()
		:	NodeIndex(INDEX_NONE)
		,	ElementIndex(INDEX_NONE)
	{}

	/** @return a boolean value representing whether the id is NULL. */
	[[nodiscard]] bool IsValidId() const
	{
		return NodeIndex != INDEX_NONE;
	}

	[[nodiscard]] uint32 GetNodeIndex() const
	{
		return NodeIndex;
	}

private:

	/** The node the element is in. */
	uint32 NodeIndex;

	/** The index of the element in the node's element array. */
	int32 ElementIndex;

	/** Initialization constructor. */
	FOctreeElementId2(uint32 InNodeIndex, int32 InElementIndex)
		:	NodeIndex(InNodeIndex)
		,	ElementIndex(InElementIndex)
	{}

	/** Implicit conversion to the element index. */
	[[nodiscard]] operator int32() const
	{
		return ElementIndex;
	}

	[[nodiscard]] friend FString LexToString(const FOctreeElementId2 ElementId)
	{
		return FString::Printf(TEXT("[%s:%s]"), *LexToString(ElementId.NodeIndex), *LexToString(ElementId.ElementIndex));
	}
};

/**
 *	An identifier for an element in the octree.
 */
class FOctreeElementId
{
public:

	template<typename, typename>
	friend class TOctree_DEPRECATED;

	/** Default constructor. */
	FOctreeElementId()
		: Node(NULL)
		, ElementIndex(INDEX_NONE)
	{}

	/** @return a boolean value representing whether the id is NULL. */
	[[nodiscard]] bool IsValidId() const
	{
		return Node != NULL;
	}

private:

	/** The node the element is in. */
	const void* Node;

	/** The index of the element in the node's element array. */
	int32 ElementIndex;

	/** Initialization constructor. */
	FOctreeElementId(const void* InNode, int32 InElementIndex)
		: Node(InNode)
		, ElementIndex(InElementIndex)
	{}

	/** Implicit conversion to the element index. */
	[[nodiscard]] operator int32() const
	{
		return ElementIndex;
	}
};