// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Concepts/Integral.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Requires.h"
#include <limits>
#include <type_traits>

/**
 * Overflow- and error-checked integer. For integer arithmetic on data from untrusted sources (like imported files),
 * especially when doing size computations. Also checks for division by zero and invalid shift amounts.
 *
 * You're not meant to use this directly. Use FGuardedInt32 or FGuardedInt64 (defined below). A typical use case
 * would be:
 * 
 *   FGuardedInt64 NumBytes = FGuardedInt32(Width) * Height * BytesPerPixel;
 *   if (NumBytes.InvalidOrGreaterThan(SizeLimit))
 *   {
 *       // Report error.
 *   }
 *   int64 NumBytesValidated = NumBytes.GetValue();
 *
 * This is a template meant to be instantiated on top of regular basic integer types. The code is written
 * so the logic is integer-size agnostic and uses just regular C++ arithmetic operations. It is assumed
 * to run on a two's complement integer platform (which is all we support, and as of C++20 is contractual).
 * You should generally use the specializations FGuardedInt32 and FGuardedInt64 below.
 *
 * Checked integers keep both the integer value and a "valid" flag. Default-constructed guarded ints
 * are invalid, and guarded integers constructed from an integer value are valid and hold that value.
 * Guarded integers are somewhat analogous to a TOptional<IntType> in semantics, and borrow some of
 * the function names.
 *
 * The main feature of guarded integers is that all arithmetic on them is overflow-checked. Any arithmetic
 * involving guarded integers results in a guarded integer, and any arithmetic involving invalid values,
 * or arithmetic resulting in overflows or other errors (such as division by zero) likewise results in
 * an invalid value. The idea is that integer arithmetic using guarded integers should be possible to
 * write very straightforwardly and without having to consider any of these special cases; if any error
 * occurred along the way, the result will be invalid. These invalid values can then be checked for and
 * handled right when the result is converted back to a regular integer.
 *
 * Some compilers provide built-ins for overflow-checked integer arithmetic for some types. We could
 * eventually use this (it's especially interesting for multiplications, since our current overflow-checking
 * algorithm is fairly expensive), but a big benefit of the current approach is that it uses nothing but
 * regular arithmetic and is type-agnostic. In particular, this makes it possible to check this implementation
 * exhaustively against a known-good reference for small integer types such as int8. It is much trickier and
 * more subtle to do good testing for larger integer types where that brute-force approach is not practical.
 * As-is, the current approach is not the fastest, but it's not presently intended to be used in contexts
 * where speed of arithmetic operations is a major concern.
 */
template<UE::CIntegral IntType>
class TGuardedInt
{
private:
	static_assert(std::is_integral_v<IntType>, "Only defined for integer types");
	typedef typename std::make_unsigned_t<IntType> UnsignedType;

	static constexpr IntType MinValue = std::numeric_limits<IntType>::min();
	static constexpr IntType MaxValue = std::numeric_limits<IntType>::max();
	static constexpr IntType NumBits = IntType((sizeof(IntType) / sizeof(char)) * 8); // Using sizeof to guess the bit count, ugh.

	IntType Value = 0;
	bool bIsValid = false;

public:
	/** Construct a TGuardedInt with an invalid value. */
	[[nodiscard]] TGuardedInt() = default;

	/** Construct a TGuardedInt from a regular signed integer value. If it's out of range, it results in an invalid value. */
	template<UE::CIntegral ValueType>
	[[nodiscard]] explicit TGuardedInt(ValueType InValue)
		: Value((IntType)InValue), bIsValid(false)
	{
		if constexpr (std::is_signed_v<ValueType>)
		{
			bIsValid = (InValue >= MinValue && InValue <= MaxValue);
		}
		else
		{
			bIsValid = InValue <= UnsignedType(MaxValue);
		}
	}

	/** Copy-construct a TGuardedInt from another of matching type. */
	[[nodiscard]] TGuardedInt(const TGuardedInt& Other) = default;

	/** Assign a TGuardedInt to another. */
	TGuardedInt& operator=(const TGuardedInt& Other) = default;

	/** @return true if current value is valid (assigned and no overflows or other errors occurred), false otherwise. */
	[[nodiscard]] bool IsValid() const
	{
		return bIsValid;
	}

	/** @return The value if valid, DefaultValue otherwise. */
	[[nodiscard]] IntType Get(const IntType DefaultValue) const
	{
		return IsValid() ? Value : DefaultValue;
	}

	/** @return Returns the value if valid, DefaultValue otherwise, but also check()s that the value is valid.
	 * Intended for cases where the value is not expected to be invalid.
	 */
	[[nodiscard]] IntType GetChecked(const IntType DefaultValue = 0) const
	{
		checkf(IsValid(), TEXT("Invalid value in TGuardedInt::GetChecked."));
		return Get(DefaultValue);
	}

	/** @return true if *this and Other are either both invalid or both valid and have the same value, false otherwise. */
	[[nodiscard]] bool operator ==(const TGuardedInt Other) const
	{
		if (bIsValid != Other.bIsValid)
		{
			return false;
		}

		return !bIsValid || Value == Other.Value;
	}

	/** @return true if *this and Other either have different "valid" states or are both valid and have different values, false otherwise (logical negation of ==). */
	[[nodiscard]] bool operator !=(const TGuardedInt Other) const
	{
		if (bIsValid != Other.bIsValid)
		{
			return true;
		}

		return bIsValid && Value != Other.Value;
	}

	// There are intentionally no overloads for the ordered comparison operators, because we have
	// to decide what to do about validity as well. Instead, do this.

	/** @return true if *this and Other are both valid so they can be compared. */
	[[nodiscard]] bool ComparisonValid(const TGuardedInt Other) const 
	{
		return bIsValid && Other.bIsValid;
	}

	/** @return true if *this and Other are both valid and *this is less than Other. */
	template<typename ValueType>
	[[nodiscard]] bool ValidAndLessThan(const ValueType Other) const
	{
		TGuardedInt CheckedOther{ Other };
		return ComparisonValid(CheckedOther) && Value < CheckedOther.Value;
	}

	/** @return true if *this and Other are both valid and *this is less than or equal to Other. */
	template<typename ValueType>
	[[nodiscard]] bool ValidAndLessOrEqual(const ValueType Other) const
	{
		TGuardedInt CheckedOther{ Other };
		return ComparisonValid(CheckedOther) && Value <= CheckedOther.Value;
	}

	/** @return true if *this and Other are both valid and *this is greater than Other. */
	template<typename ValueType>
	[[nodiscard]] bool ValidAndGreaterThan(const ValueType Other) const
	{
		TGuardedInt CheckedOther{ Other };
		return ComparisonValid(CheckedOther) && Value > CheckedOther.Value;
	}

	/** @return true if *this and Other are both valid and *this is greater than or equal to Other. */
	template<typename ValueType>
	[[nodiscard]] bool ValidAndGreaterOrEqual(const ValueType Other) const
	{
		TGuardedInt CheckedOther{ Other };
		return ComparisonValid(CheckedOther) && Value >= CheckedOther.Value;
	}

	/** @return true if either of *this or Other are invalid or *this is less than Other. */
	template<typename ValueType>
	[[nodiscard]] bool InvalidOrLessThan(const ValueType Other) const
	{
		TGuardedInt CheckedOther{ Other };
		return !ComparisonValid(CheckedOther) || Value < CheckedOther.Value;
	}

	/** @return true if either of *this or Other are invalid or *this is less than or equal to Other. */
	template<typename ValueType>
	[[nodiscard]] bool InvalidOrLessOrEqual(const ValueType Other) const
	{
		TGuardedInt CheckedOther{ Other };
		return !ComparisonValid(CheckedOther) || Value <= CheckedOther.Value;
	}

	/** @return true if either of *this or Other are invalid or *this is greater than Other. */
	template<typename ValueType>
	[[nodiscard]] bool InvalidOrGreaterThan(const ValueType Other) const
	{
		TGuardedInt CheckedOther{ Other };
		return !ComparisonValid(CheckedOther) || Value > CheckedOther.Value;
	}

	/** @return true if either of *this or Other are invalid or *this is greater than or equal to Other. */
	template<typename ValueType>
	[[nodiscard]] bool InvalidOrGreaterOrEqual(const ValueType Other) const
	{
		TGuardedInt CheckedOther{ Other };
		return !ComparisonValid(CheckedOther) || Value >= CheckedOther.Value;
	}

	// Arithmetic operations

	/** @return The negated value. */
	[[nodiscard]] TGuardedInt operator-() const
	{
		if constexpr (std::is_signed_v<IntType>)
		{
			// Unary negation (for two's complement) overflows iff the operand is MinValue.
			return (bIsValid && Value > MinValue) ? TGuardedInt(-Value) : TGuardedInt();
		}
		else
		{
			// Negating anything but zero overflows unsigned integers.
			return (bIsValid && Value == 0) ? TGuardedInt(-Value) : TGuardedInt();
		}
	}

	/** @return The sum of the two operands. */
	[[nodiscard]] TGuardedInt operator +(const TGuardedInt Other) const
	{
		// Any sum involving an invalid value is invalid.
		if (!bIsValid || !Other.bIsValid)
		{
			return TGuardedInt();
		}

		IntType Result;
		return FPlatformMath::AddAndCheckForOverflow<IntType>(Value, Other.Value, Result)
			? TGuardedInt(Result)
			: TGuardedInt();
	}

	/** @return The difference between the two operands. */
	[[nodiscard]] TGuardedInt operator -(const TGuardedInt Other) const
	{
		// Any difference involving an invalid value is invalid.
		if (!bIsValid || !Other.bIsValid)
		{
			return TGuardedInt();
		}
		
		IntType Result;
		return FPlatformMath::SubtractAndCheckForOverflow<IntType>(Value, Other.Value, Result)
			? TGuardedInt(Result)
			: TGuardedInt();
	}

	/** @return The product of the two operands. */
	[[nodiscard]] TGuardedInt operator *(const TGuardedInt Other) const
	{
		// Any product involving invalid values is invalid.
		if (!bIsValid || !Other.bIsValid)
		{
			return TGuardedInt();
		}

		IntType Result;
		return FPlatformMath::MultiplyAndCheckForOverflow<IntType>(Value, Other.Value, Result)
			? TGuardedInt(Result)
			: TGuardedInt();
	}

	/** @return The quotient when dividing *this by Other. */
	[[nodiscard]] TGuardedInt operator /(const TGuardedInt Other) const
	{
		// Any quotient involving invalid values is invalid.
		if (!bIsValid || !Other.bIsValid)
		{
			return TGuardedInt();
		}

		// Luckily for us, division generally makes things smaller, so there's only two things to watch
		// out for: division by zero is not allowed, and division of MinValue by -1 would give -MinValue
		// which overflows. All other combinations are fine.
		if (Other.Value == 0 || (std::is_signed_v<IntType> && Value == MinValue && Other.Value == -1))
		{
			return TGuardedInt();
		}

		return TGuardedInt(Value / Other.Value);
	}

	/** @return The remainder when dividing *this by Other. */
	[[nodiscard]] TGuardedInt operator %(const TGuardedInt Other) const
	{
		// Any quotient involving invalid values is invalid.
		if (!bIsValid || !Other.bIsValid)
		{
			return TGuardedInt();
		}

		// Same error cases as for division.
		if (Other.Value == 0 || (std::is_signed_v<IntType> && Value == MinValue && Other.Value == -1))
		{
			return TGuardedInt();
		}

		return TGuardedInt(Value % Other.Value);
	}

	/** @return This value bitwise left-shifted by the operand. */
	[[nodiscard]] TGuardedInt operator <<(const TGuardedInt Other) const
	{
		// Any shift involving invalid values is invalid.
		if (!bIsValid || !Other.bIsValid)
		{
			return TGuardedInt();
		}

		// Left-shifts by negative values or >= the width of the type are always invalid.
		if (Other.Value < 0 || Other.Value >= NumBits)
		{
			return TGuardedInt();
		}

		if constexpr (std::is_signed_v<IntType>)
		{
			const int ShiftAmount = Other.Value;

			// Once again, taking our overflow-prone expression and using algebra to find
			// a form that doesn't overflow:
			//
			// MinValue <= a * 2^b <= MaxValue
			// <=> MinValue * 2^(-b) <= a <= MaxValue * 2^(-b)
			//
			// The LHS is exact because MinValue is -2^(NumBits - 1) for two's complement,
			// and we just ensured that 0 <= b < NumBits (with b integer).
			//
			// The RHS has a fractional part whereas a is integer; therefore, we can
			// substitute floor(MaxValue * 2^(-b)) for the RHS without changing the result.
			//
			// And that gives us our test!
			return ((MinValue >> ShiftAmount) <= Value && Value <= (MaxValue >> ShiftAmount)) ? TGuardedInt(Value << ShiftAmount) : TGuardedInt();
		}
		else
		{
			return (static_cast<IntType>(Value << Other.Value) >> Other.Value) == Value
				? TGuardedInt(static_cast<IntType>(Value << Other.Value))
				: TGuardedInt();
		}
	}

	/** @return This value bitwise right-shifted by the operand. */
	[[nodiscard]] TGuardedInt operator >>(const TGuardedInt Other) const
	{
		// Any shift involving invalid values is invalid.
		if (!bIsValid || !Other.bIsValid)
		{
			return TGuardedInt();
		}

		// Right-shifts by negative values or >= the width of the type are always invalid.
		if (Other.Value < 0 || Other.Value >= NumBits)
		{
			return TGuardedInt();
		}

		// Right-shifts don't have any overflow conditions, so we're good!
		return TGuardedInt(Value >> Other.Value);
	}

	/** @return The absolute value of *this. */
	[[nodiscard]] TGuardedInt Abs() const
	{
		if (!bIsValid)
		{
			return TGuardedInt();
		}

		if constexpr (std::is_signed_v<IntType>)
		{
			// Note the absolute value of MinValue overflows, so this is not completely trivial.
			// Can't just use TGuardedInt(abs(Value)) here!
			return (Value < 0) ? -*this : *this;
		}
		else
		{
			// Abs on unsigned integers is the identity function, and can't overflow.
			return TGuardedInt(Value);
		}
	}

	// Mixed-type operators and assignment operators reduce to the base operators systematically
#define UE_GUARDED_SIGNED_INT_IMPL_BINARY_OPERATOR(OP) \
	/* Mixed-type expressions that coerce both operands to guarded ints */ \
	TGuardedInt operator OP(IntType InB) const { return *this OP TGuardedInt(InB); } \
	friend TGuardedInt operator OP(IntType InA, TGuardedInt InB) { return TGuardedInt(InA) OP InB; } \
	/* Assignment operators, direct and mixed */ \
	TGuardedInt& operator OP##=(TGuardedInt InB) { return *this = *this OP InB; } \
	TGuardedInt& operator OP##=(IntType InB) { return *this = *this OP TGuardedInt(InB); } \
	/* end */

	UE_GUARDED_SIGNED_INT_IMPL_BINARY_OPERATOR(+)
	UE_GUARDED_SIGNED_INT_IMPL_BINARY_OPERATOR(-)
	UE_GUARDED_SIGNED_INT_IMPL_BINARY_OPERATOR(*)
	UE_GUARDED_SIGNED_INT_IMPL_BINARY_OPERATOR(/)
	UE_GUARDED_SIGNED_INT_IMPL_BINARY_OPERATOR(%)
	UE_GUARDED_SIGNED_INT_IMPL_BINARY_OPERATOR(<<)
	UE_GUARDED_SIGNED_INT_IMPL_BINARY_OPERATOR(>>)

#undef UE_GUARDED_SIGNED_INT_IMPL_BINARY_OPERATOR
};

/** Legacy alias for the previously signed-integer-only implementation. */
template<typename SignedType>
using TGuardedSignedInt = TGuardedInt<SignedType>;

/** Guarded 32-bit integer class. Used to deal with integer data from untrusted sources in size computations etc. */
using FGuardedInt32 = TGuardedInt<int32>;

/** Guarded 64-bit integer class. Used to deal with integer data from untrusted sources in size computations etc. */
using FGuardedInt64 = TGuardedInt<int64>;
