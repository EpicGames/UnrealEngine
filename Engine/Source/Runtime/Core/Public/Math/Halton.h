// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"


/** [ Halton 1964, "Radical-inverse quasi-random point sequence" ] */
[[nodiscard]] constexpr float Halton(int32 Index, int32 Base)
{
	float Result = 0.0f;
	float InvBase = 1.0f / float(Base);
	float Fraction = InvBase;
	while (Index > 0)
	{
		Result += float(Index % Base) * Fraction;
		Index /= Base;
		Fraction *= InvBase;
	}
	return Result;
}
