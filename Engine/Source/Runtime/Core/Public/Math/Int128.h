// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Serialization/Archive.h"

class FUInt128
{
private:
	/** Internal values representing this number */
	uint64 Hi;
	uint64 Lo;

	inline uint32 AddInternal(uint32 A, uint32 B, uint32& Carry)
	{
		uint64 Result = (uint64)A + (uint64)B + (uint64)Carry;
		Carry = (Result >> 32) & 1;
		return (uint32)Result;
	}

	inline uint32 MultiplyInternal(uint32 Multiplicand, uint32 Multiplier, uint32& Carry)
	{
		uint64 Result;
		Result = ((uint64)Multiplicand * (uint64)Multiplier) + (uint64)Carry;
		Carry = Result >> 32;
		return (uint32)Result;
	}

	inline uint32 DivideInternal(uint32 Dividend, uint32 Divisor, uint32& Remainder)
	{
		uint64 Value = ((uint64)Remainder << 32) | Dividend;
		Remainder = Value % Divisor;
		return (uint32)(Value / Divisor);
	}

public:
	/** Gets internal quad parts. */
	[[nodiscard]] inline uint32 GetQuadPart(uint32 Part) const
	{
		switch (Part)
		{
		case 3: return Hi >> 32;
		case 2: return (uint32)Hi;
		case 1: return Lo >> 32;
		case 0: return (uint32)Lo;
		default: check(0);
		}
		return 0;
	}

	/** Sets internal quad parts. */
	inline void SetQuadPart(uint32 Part, uint32 Value)
	{
		switch (Part)
		{
		case 3: Hi = (Hi & 0x00000000ffffffffull) | ((uint64)Value << 32); break;
		case 2: Hi = (Hi & 0xffffffff00000000ull) | Value; break;
		case 1: Lo = (Lo & 0x00000000ffffffffull) | ((uint64)Value << 32); break;
		case 0: Lo = (Lo & 0xffffffff00000000ull) | Value; break;
		default: check(0);
		}
	}

	/** Sets this number to 0. */
	UE_FORCEINLINE_HINT void Zero()
	{
		Hi = Lo = 0;
	}

	/** Initializes this number with a pair of 64 bit integer values. */
	inline void Set(uint64 InHi, uint64 InLo)
	{
		Hi = InHi;
		Lo = InLo;
	}

	/** Default constructors. */
    [[nodiscard]] UE_FORCEINLINE_HINT FUInt128(const FUInt128&) = default;
    [[nodiscard]] UE_FORCEINLINE_HINT FUInt128(FUInt128&&) = default;
    UE_FORCEINLINE_HINT FUInt128& operator=(FUInt128 const&) = default;
    UE_FORCEINLINE_HINT FUInt128& operator=(FUInt128&&) = default;

	/** Default constructor. Initializes the number to zero. */
	[[nodiscard]] inline FUInt128()
		: Hi(0)
		, Lo(0)
	{}

	/** Constructor. Initializes this uint128 with a uint64 value. */
	[[nodiscard]] inline FUInt128(uint64 A)
		: Hi(0)
		, Lo(A)
	{}

	/** Constructor. Initializes this uint128 with two uint64 values. */
	[[nodiscard]] inline FUInt128(uint64 A, uint64 B)
		: Hi(A)
		, Lo(B)
	{}

	/** Constructor. Initializes this uint128 with four uint32 values. */
	[[nodiscard]] inline FUInt128(uint32 A, uint32 B, uint32 C, uint32 D)
		: Hi(((uint64)A << 32) | B)
		, Lo(((uint64)C << 32) | D)
	{}

	/** this > Other */
	[[nodiscard]] inline bool IsGreater(const FUInt128& Other) const
	{
		if (Hi == Other.Hi)
		{
			return Lo > Other.Lo;
		}
		return Hi > Other.Hi;
	}

	/** this >= Other */
	[[nodiscard]] inline bool IsGreaterOrEqual(const FUInt128& Other) const
	{
		if (Hi == Other.Hi)
		{
			return Lo >= Other.Lo;
		}
		return Hi >= Other.Hi;
	}

	/** this < Other */
	[[nodiscard]] inline bool IsLess(const FUInt128& Other) const
	{
		if (Hi == Other.Hi)
		{
			return Lo < Other.Lo;
		}
		return Hi < Other.Hi;
	}

	/** this <= Other */
	[[nodiscard]] inline bool IsLessOrEqual(const FUInt128& Other) const
	{
		if (Hi == Other.Hi)
		{
			return Lo <= Other.Lo;
		}
		return Hi <= Other.Hi;
	}

	/** this == Other */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsEqual(const FUInt128& Other) const
	{
		return (Hi == Other.Hi) && (Lo == Other.Lo);
	}

	/** Add an unsigned 32bit value */
	inline FUInt128 Add(uint32 Value)
	{
		uint32 Carry = 0;

		FUInt128 Result;
		Result.SetQuadPart(0, AddInternal(GetQuadPart(0), Value, Carry));
		Result.SetQuadPart(1, AddInternal(GetQuadPart(1), 0, Carry));
		Result.SetQuadPart(2, AddInternal(GetQuadPart(2), 0, Carry));
		Result.SetQuadPart(3, AddInternal(GetQuadPart(3), 0, Carry));
		return Result;
	}

	/* Subtract an unsigned 32bit value */
	inline FUInt128 Sub(uint32 Value)
	{
		uint32 Carry = 0;

		uint32 AddValue = ~Value + 1;
		uint32 SignExtend = (AddValue >> 31) ? ~0 : 0;

		FUInt128 Result;
		Result.SetQuadPart(0, AddInternal(GetQuadPart(0), AddValue, Carry));
		Result.SetQuadPart(1, AddInternal(GetQuadPart(1), SignExtend, Carry));
		Result.SetQuadPart(2, AddInternal(GetQuadPart(2), SignExtend, Carry));
		Result.SetQuadPart(3, AddInternal(GetQuadPart(3), SignExtend, Carry));
		return Result;
	}

	/** Multiply by an unsigned 32bit value */
	inline FUInt128 Multiply(uint32 Multiplier)
	{
		uint32 Carry = 0;

		FUInt128 Result;
		Result.SetQuadPart(0, MultiplyInternal(GetQuadPart(0), Multiplier, Carry));
		Result.SetQuadPart(1, MultiplyInternal(GetQuadPart(1), Multiplier, Carry));
		Result.SetQuadPart(2, MultiplyInternal(GetQuadPart(2), Multiplier, Carry));
		Result.SetQuadPart(3, MultiplyInternal(GetQuadPart(3), Multiplier, Carry));
		return Result;
	}

	/** Divide by an unsigned 32bit value */
	inline FUInt128 Divide(uint32 Divisor, uint32& Remainder)
	{
		Remainder = 0;

		FUInt128 Result;
		Result.SetQuadPart(3, DivideInternal(GetQuadPart(3), Divisor, Remainder));
		Result.SetQuadPart(2, DivideInternal(GetQuadPart(2), Divisor, Remainder));
		Result.SetQuadPart(1, DivideInternal(GetQuadPart(1), Divisor, Remainder));
		Result.SetQuadPart(0, DivideInternal(GetQuadPart(0), Divisor, Remainder));
		return Result;
	}

	/**
	 * Comparison operators
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator>(const FUInt128& Other) const
	{
		return IsGreater(Other);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator>=(const FUInt128& Other) const
	{
		return IsGreaterOrEqual(Other);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const FUInt128& Other) const
	{
		return IsEqual(Other);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator<(const FUInt128& Other) const
	{
		return IsLess(Other);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator<=(const FUInt128& Other) const
	{
		return IsLessOrEqual(Other);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool IsZero() const
	{
		return !(Hi | Lo);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool IsGreaterThanZero() const
	{
		return !IsZero();
	}

	/**
	 * Arithmetic operators
	 */
	inline FUInt128& operator+=(uint32 Other)
	{
		*this = Add(Other);
		return *this;
	}

	inline FUInt128& operator-=(uint32 Other)
	{
		*this = Sub(Other);
		return *this;
	}

	inline FUInt128& operator*=(uint32 Other)
	{
		*this = Multiply(Other);
		return *this;
	}

	inline FUInt128& operator/=(uint32 Other)
	{
		uint32 Remainder;
		*this = Divide(Other, Remainder);
		return *this;
	}

	/**
	 * Serialization
	 */
	friend FArchive& operator<<(FArchive& Ar, FUInt128& Value)
	{
		return Ar << Value.Hi << Value.Lo;
	}

	/**
	 * Casting
	 */
	inline operator uint64() const
	{
		check(Hi == 0);
		return Lo;
	}
};