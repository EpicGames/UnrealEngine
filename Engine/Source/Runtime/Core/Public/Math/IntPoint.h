// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Parse.h"
#include "Math/MathFwd.h" // IWYU pragma: export
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/TypeHash.h"
#include "Misc/LargeWorldCoordinatesSerializer.h"

namespace UE::Math
{

/**
 * Structure for integer points in 2-d space.
 *
 * @todo Docs: The operators need better documentation, i.e. what does it mean to divide a point?
 */
template <typename InIntType>
struct TIntPoint
{
	using IntType = InIntType;
	static_assert(std::is_integral_v<IntType>, "An integer type is required.");

	union
	{
		struct
		{
			/** Holds the point's x-coordinate. */
			IntType X;

			/** Holds the point's y-coordinate. */
			IntType Y;
		};

		UE_DEPRECATED(all, "For internal use only")
		IntType XY[2];
	};

	/** An integer point with zeroed values. */
	static const TIntPoint ZeroValue;

	/** An integer point with INDEX_NONE values. */
	static const TIntPoint NoneValue;

	/** Default constructor (no initialization). */
	[[nodiscard]] TIntPoint() = default;

	/**
	 * Create and initialize a new instance with the specified coordinates.
	 *
	 * @param InX The x-coordinate.
	 * @param InY The y-coordinate.
	 */
[[nodiscard]] TIntPoint(IntType InX, IntType InY)
		: X(InX)
		, Y(InY)
	{
	}

	/**
	 * Create and initialize a new instance with a single int.
	 * Both X and Y will be initialized to this value
	 *
	 * @param InXY The x and y-coordinate.
	 */
	[[nodiscard]] TIntPoint(IntType InXY)
		: X(InXY)
		, Y(InXY)
	{
	}

	/**
	 * Create and initialize a new instance to zero.
	 *
	 * @param EForceInit Force init enum
	 */
	[[nodiscard]] explicit TIntPoint(EForceInit)
		: X(0)
		, Y(0)
	{
	}

	/**
	 * Converts to another int type. Checks that the cast will succeed.
	 */
	template <typename OtherIntType>
	[[nodiscard]] explicit TIntPoint(TIntPoint<OtherIntType> Other)
		: X(IntCastChecked<IntType>(Other.X))
		, Y(IntCastChecked<IntType>(Other.Y))
	{
	}

	// Workaround for clang deprecation warnings for deprecated XY member in implicitly-defined special member functions
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TIntPoint(TIntPoint&&) = default;
	TIntPoint(const TIntPoint&) = default;
	TIntPoint& operator=(TIntPoint&&) = default;
	TIntPoint& operator=(const TIntPoint&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Get specific component of a point.
	 *
	 * @param PointIndex Index of point component.
	 * @return const reference to component.
	 */
	[[nodiscard]] const IntType& operator()(int32 PointIndex) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XY[PointIndex];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Get specific component of a point.
	 *
	 * @param PointIndex Index of point component
	 * @return reference to component.
	 */
	[[nodiscard]] IntType& operator()(int32 PointIndex)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XY[PointIndex];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Compare two points for equality.
	 *
	 * @param Other The other int point being compared.
	 * @return true if the points are equal, false otherwise.
	 */
	[[nodiscard]] bool operator==(const TIntPoint& Other) const
	{
		return X == Other.X && Y == Other.Y;
	}

	/**
	 * Compare two points for inequality.
	 *
	 * @param Other The other int point being compared.
	 * @return true if the points are not equal, false otherwise.
	 */
	[[nodiscard]] bool operator!=(const TIntPoint& Other) const
	{
		return (X != Other.X) || (Y != Other.Y);
	}

	/**
	 * Scale this point.
	 *
	 * @param Scale What to multiply the point by.
	 * @return Reference to this point after multiplication.
	 */
	TIntPoint& operator*=(IntType Scale)
	{
		X *= Scale;
		Y *= Scale;

		return *this;
	}

	/**
	 * Divide this point by a scalar.
	 *
	 * @param Divisor What to divide the point by.
	 * @return Reference to this point after division.
	 */
	TIntPoint& operator/=(IntType Divisor)
	{
		X /= Divisor;
		Y /= Divisor;

		return *this;
	}

	/**
	 * Add another point component-wise to this point.
	 *
	 * @param Other The point to add to this point.
	 * @return Reference to this point after addition.
	 */
	TIntPoint& operator+=(const TIntPoint& Other)
	{
		X += Other.X;
		Y += Other.Y;

		return *this;
	}

	/**
	 * Multiply another point component-wise from this point.
	 *
	 * @param Other The point to multiply with this point.
	 * @return Reference to this point after multiplication.
	 */
	TIntPoint& operator*=(const TIntPoint& Other)
	{
		X *= Other.X;
		Y *= Other.Y;

		return *this;
	}

	/**
	 * Subtract another point component-wise from this point.
	 *
	 * @param Other The point to subtract from this point.
	 * @return Reference to this point after subtraction.
	 */
	TIntPoint& operator-=(const TIntPoint& Other)
	{
		X -= Other.X;
		Y -= Other.Y;

		return *this;
	}

	/**
	 * Divide this point component-wise by another point.
	 *
	 * @param Other The point to divide with.
	 * @return Reference to this point after division.
	 */
	TIntPoint& operator/=(const TIntPoint& Other)
	{
		X /= Other.X;
		Y /= Other.Y;

		return *this;
	}

	/**
	 * Get the result of scaling on this point.
	 *
	 * @param Scale What to multiply the point by.
	 * @return A new scaled int point.
	 */
	[[nodiscard]] TIntPoint operator*(IntType Scale) const
	{
		return TIntPoint(*this) *= Scale;
	}

	/**
	 * Get the result of division on this point.
	 *
	 * @param Divisor What to divide the point by.
	 * @return A new divided int point.
	 */
	[[nodiscard]] TIntPoint operator/(IntType Divisor) const
	{
		return TIntPoint(*this) /= Divisor;
	}

	/**
	 * Get the result of addition on this point.
	 *
	 * @param Other The other point to add to this.
	 * @return A new combined int point.
	 */
	[[nodiscard]] TIntPoint operator+(const TIntPoint& Other) const
	{
		return TIntPoint(*this) += Other;
	}

	/**
	 * Get the result of subtraction from this point.
	 *
	 * @param Other The other point to subtract from this.
	 * @return A new subtracted int point.
	 */
	[[nodiscard]] TIntPoint operator-(const TIntPoint& Other) const
	{
		return TIntPoint(*this) -= Other;
	}

	/**
	 * Get the result of multiplication on this point.
	 *
	 * @param Other The point to multiply with this point.
	 * @return A new multiplied int point
	 */
	[[nodiscard]] TIntPoint operator*(const TIntPoint& Other) const
	{
		return TIntPoint(*this) *= Other;
	}

	/**
	 * Get the result of division on this point.
	 *
	 * @param Other The other point to subtract from this.
	 * @return A new subtracted int point.
	 */
	[[nodiscard]] TIntPoint operator/(const TIntPoint& Other) const
	{
		return TIntPoint(*this) /= Other;
	}

	/**
	 * Get specific component of the point.
	 *
	 * @param Index the index of point component
	 * @return reference to component.
	 */
	[[nodiscard]] IntType& operator[](IntType Index)
	{
		check(Index >= 0 && Index < 2);
		return ((Index == 0) ? X : Y);
	}

	/**
	 * Get specific component of the point.
	 *
	 * @param Index the index of point component
	 * @return copy of component value.
	 */
	[[nodiscard]] IntType operator[](IntType Index) const
	{
		check(Index >= 0 && Index < 2);
		return ((Index == 0) ? X : Y);
	}

	/**
	 * Get the component-wise min of two points.
	 *
	 * @see ComponentMax, GetMax
	 */
	[[nodiscard]] TIntPoint ComponentMin(const TIntPoint& Other) const
	{
		return TIntPoint(FMath::Min(X, Other.X), FMath::Min(Y, Other.Y));
	}

	/**
	 * Get the component-wise max of two points.
	 *
	 * @see ComponentMin, GetMin
	 */
	[[nodiscard]] TIntPoint ComponentMax(const TIntPoint& Other) const
	{
		return TIntPoint(FMath::Max(X, Other.X), FMath::Max(Y, Other.Y));
	}

	/**
	 * Get the larger of the point's two components.
	 *
	 * @return The maximum component of the point.
	 * @see GetMin, Size, SizeSquared
	 */
	[[nodiscard]] IntType GetMax() const
	{
		return FMath::Max(X, Y);
	}

	/**
	 * Get the smaller of the point's two components.
	 *
	 * @return The minimum component of the point.
	 * @see GetMax, Size, SizeSquared
	 */
	[[nodiscard]] IntType GetMin() const
	{
		return FMath::Min(X, Y);
	}

	/**
	 * Get the distance of this point from (0,0).
	 *
	 * @return The distance of this point from (0,0).
	 * @see GetMax, GetMin, SizeSquared
	 */
	[[nodiscard]] IntType Size() const
	{
		int64 LocalX64 = (int64)X;
		int64 LocalY64 = (int64)Y;
		return IntType(FMath::Sqrt(double(LocalX64 * LocalX64 + LocalY64 * LocalY64)));
	}

	/**
	 * Get the squared distance of this point from (0,0).
	 *
	 * @return The squared distance of this point from (0,0).
	 * @see GetMax, GetMin, Size
	 */
	[[nodiscard]] IntType SizeSquared() const
	{
		return X * X + Y * Y;
	}

	/**
	 * Get a textual representation of this point.
	 *
	 * @return A string describing the point.
	 */
	[[nodiscard]] FString ToString() const
	{
		return FString::Printf(TEXT("X=%s Y=%s"), *LexToString(X), *LexToString(Y));
	}

	/**
	 * Initialize this FIntPoint based on an FString. The String is expected to contain X=, Y=
	 * The FIntPoint will be bogus when InitFromString returns false.
	 *
	 * @param	InSourceString	FString containing the color values.
	 * @return true if the X,Y values were read successfully; false otherwise.
	 */
	bool InitFromString(const FString& InSourceString)
	{
		X = Y = 0;

		// The initialization is only successful if the X and Y values can all be parsed from the string
		const bool bSuccessful = FParse::Value(*InSourceString, TEXT("X="), X) && FParse::Value(*InSourceString, TEXT("Y="), Y);

		return bSuccessful;
	}

	/**
	 * Divide an int point and round up the result.
	 *
	 * @param lhs The int point being divided.
	 * @param Divisor What to divide the int point by.
	 * @return A new divided int point.
	 * @see DivideAndRoundDown
	 */
	[[nodiscard]] static TIntPoint DivideAndRoundUp(TIntPoint lhs, IntType Divisor)
	{
		return TIntPoint(FMath::DivideAndRoundUp(lhs.X, Divisor), FMath::DivideAndRoundUp(lhs.Y, Divisor));
	}

	[[nodiscard]] static TIntPoint DivideAndRoundUp(TIntPoint lhs, TIntPoint Divisor)
	{
		return TIntPoint(FMath::DivideAndRoundUp(lhs.X, Divisor.X), FMath::DivideAndRoundUp(lhs.Y, Divisor.Y));
	}

	/**
	 * Divide an int point and round down the result.
	 *
	 * @param lhs The int point being divided.
	 * @param Divisor What to divide the int point by.
	 * @return A new divided int point.
	 * @see DivideAndRoundUp
	 */
	[[nodiscard]] static TIntPoint DivideAndRoundDown(TIntPoint lhs, IntType Divisor)
	{
		return TIntPoint(FMath::DivideAndRoundDown(lhs.X, Divisor), FMath::DivideAndRoundDown(lhs.Y, Divisor));
	}

	[[nodiscard]] static TIntPoint DivideAndRoundDown(TIntPoint lhs, TIntPoint Divisor)
	{
		return TIntPoint(FMath::DivideAndRoundDown(lhs.X, Divisor.X), FMath::DivideAndRoundDown(lhs.Y, Divisor.Y));
	}

	/**
	 * Get number of components point has.
	 *
	 * @return number of components point has.
	 */
	[[nodiscard]] static int32 Num()
	{
		return 2;
	}

	/**
	 * Serialize the point.
	 *
	 * @param Ar The archive to serialize into.
	 * @param Point The point to serialize.
	 * @return Reference to the Archive after serialization.
	 */
	friend FArchive& operator<<(FArchive& Ar, TIntPoint& Point)
	{
		return Ar << Point.X << Point.Y;
	}

	/**
	 * Serialize the point.
	 *
	 * @param Slot The structured archive slot to serialize into.
	 * @param Point The point to serialize.
	 */
	friend void operator<<(FStructuredArchive::FSlot Slot, TIntPoint& Point)
	{
		FStructuredArchive::FRecord Record = Slot.EnterRecord();
		Record << SA_VALUE(TEXT("X"), Point.X) << SA_VALUE(TEXT("Y"), Point.Y);
	}

	/**
	 * Serialize the point.
	 *
	 * @param Ar The archive to serialize into.
	 * @return true on success, false otherwise.
	 */
	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		if constexpr (std::is_same_v<IntType, int32>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, IntPoint, Int32Point, Int64Point);
		}
		else if constexpr (std::is_same_v<IntType, int64>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, IntPoint, Int64Point, Int32Point);
		}
		else if constexpr (std::is_same_v<IntType, uint32>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, UintPoint, Uint32Point, Uint64Point);
		}
		else if constexpr (std::is_same_v<IntType, uint64>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, UintPoint, Uint64Point, Uint32Point);
		}
		else
		{
			static_assert(sizeof(IntType) == 0, "Unimplemented");
			return false;
		}
	}
};

template <typename IntType>
const TIntPoint<IntType> TIntPoint<IntType>::ZeroValue(0, 0);

template <typename IntType>
const TIntPoint<IntType> TIntPoint<IntType>::NoneValue(static_cast<IntType>(INDEX_NONE), static_cast<IntType>(INDEX_NONE));

template <typename IntType>
uint32 GetTypeHash(const TIntPoint<IntType>& InPoint)
{
	return HashCombine(::GetTypeHash(InPoint.X), ::GetTypeHash(InPoint.Y));
}

template <>
inline FString TIntPoint<int64>::ToString() const
{
	return FString::Printf(TEXT("X=%lld Y=%lld"), X, Y);
}

template <>
inline FString TIntPoint<int32>::ToString() const
{
	return FString::Printf(TEXT("X=%d Y=%d"), X, Y);
}

template <>
inline FString TIntPoint<int16>::ToString() const
{
	return FString::Printf(TEXT("X=%d Y=%d"), X, Y);
}

template <>
inline FString TIntPoint<int8>::ToString() const
{
	return FString::Printf(TEXT("X=%d Y=%d"), X, Y);
}

template <>
inline FString TIntPoint<uint64>::ToString() const
{
	return FString::Printf(TEXT("X=%llu Y=%llu"), X, Y);
}

template <>
inline FString TIntPoint<uint32>::ToString() const
{
	return FString::Printf(TEXT("X=%u Y=%u"), X, Y);
}

template <>
inline FString TIntPoint<uint16>::ToString() const
{
	return FString::Printf(TEXT("X=%u Y=%u"), X, Y);
}

template <>
inline FString TIntPoint<uint8>::ToString() const
{
	return FString::Printf(TEXT("X=%u Y=%u"), X, Y);
}

} //! namespace UE::Math

template <> struct TIsPODType<FInt32Point>  { enum { Value = true }; };
template <> struct TIsPODType<FUint32Point> { enum { Value = true }; };

template<> struct TIsUECoreVariant<FInt32Point>  { enum { Value = true }; };
template<> struct TIsUECoreVariant<FUint32Point> { enum { Value = true }; };

template <> struct TIsPODType<FInt64Point> { enum { Value = true }; };
template <> struct TIsPODType<FUint64Point> { enum { Value = true }; };

template<> struct TIsUECoreVariant<FInt64Point> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FUint64Point> { enum { Value = true }; };
