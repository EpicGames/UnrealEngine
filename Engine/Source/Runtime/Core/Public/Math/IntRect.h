// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/MathFwd.h" // IWYU pragma: export
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Math/IntPoint.h"
#include "Math/Vector2D.h"

namespace UE::Math
{

/**
 * Structure for integer rectangles in 2-d space.
 * 
 * Represented as half-open bounds [min, max)
 *
 * @todo Docs: The operators need better documentation, i.e. what does it mean to divide a rectangle?
 */
template <typename InIntType>
struct TIntRect
{
	using IntType = InIntType;
	using IntPointType = TIntPoint<IntType>;
	static_assert(std::is_integral_v<IntType>, "Only an integer types are supported.");

	union
	{
		struct
		{
			/** Holds the first pixel line/row (like in Win32 RECT). */
			IntPointType Min;

			/** Holds the last pixel line/row (like in Win32 RECT). */
			IntPointType Max;
		};

		UE_DEPRECATED(all, "For internal use only")
		IntPointType MinMax[2];
	};

	/** Constructor */
	[[nodiscard]] TIntRect()
		: Min(ForceInit)
		, Max(ForceInit)
	{}

	/**
	 * Constructor
	 *
	 * @param X0 Minimum X coordinate (inclusive)
	 * @param Y0 Minimum Y coordinate (inclusive)
	 * @param X1 Maximum X coordinate (exclusive)
	 * @param Y1 Maximum Y coordinate (exclusive)
	 * 
	 * @note  
	 */
	[[nodiscard]] TIntRect(IntType X0, IntType Y0, IntType X1, IntType Y1)
		: Min(X0, Y0)
		, Max(X1, Y1)
	{
	}

	/**
	 * Constructor
	 *
	 * @param InMin Minimum Point (inclusive)
	 * @param InMax Maximum Point (exclusive)
	 */
	[[nodiscard]] TIntRect(IntPointType InMin, IntPointType InMax)
		: Min(InMin)
		, Max(InMax)
	{
	}

	[[nodiscard]] TIntRect(const TIntRect& Other)
	{
		*this = Other;
	}

	TIntRect& operator=(const TIntRect& Other)
	{
		Min = Other.Min;
		Max = Other.Max;
		return *this;
	}

	/**
	 * Converts to another int type. Checks that the cast will succeed.
	 */
	template <typename OtherIntType>
	[[nodiscard]] explicit TIntRect(TIntRect<OtherIntType> Other)
		: Min(IntPointType(Other.Min))
		, Max(IntPointType(Other.Max))
	{
	}

	/**
	 * Gets a specific point in this rectangle.
	 *
	 * @param PointIndex Index of Point in rectangle.
	 * @return Const reference to point in rectangle.
	 */
	[[nodiscard]] const TIntRect& operator()(int32 PointIndex) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MinMax[PointIndex];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Gets a specific point in this rectangle.
	 *
	 * @param PointIndex Index of Point in rectangle.
	 * @return Reference to point in rectangle.
	 */
	[[nodiscard]] TIntRect& operator()(int32 PointIndex)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MinMax[PointIndex];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Compares Rectangles for equality.
	 *
	 * @param Other The Other Rectangle for comparison.
	 * @return true if the rectangles are equal, false otherwise..
	 */
	[[nodiscard]] bool operator==(const TIntRect& Other) const
	{
		return Min == Other.Min && Max == Other.Max;
	}

	/**
	 * Compares Rectangles for inequality.
	 *
	 * @param Other The Other Rectangle for comparison.
	 * @return true if the rectangles are not equal, false otherwise..
	 */
	[[nodiscard]] bool operator!=(const TIntRect& Other) const
	{
		return Min != Other.Min || Max != Other.Max;
	}

	/**
	 * Applies scaling to this rectangle.
	 *
	 * @param Scale What to multiply the rectangle by.
	 * @return Reference to this rectangle after scaling.
	 */
	TIntRect& operator*=(IntType Scale)
	{
		Min *= Scale;
		Max *= Scale;

		return *this;
	}

	/**
	 * Adds a point to this rectangle.
	 *
	 * @param Point The point to add onto both points in the rectangle.
	 * @return Reference to this rectangle after addition.
	 */
	TIntRect& operator+=(const IntPointType& Point)
	{
		Min += Point;
		Max += Point;

		return *this;
	}

	/**
	 * Subtracts a point from this rectangle.
	 *
	 * @param Point The point to subtract from both points in the rectangle.
	 * @return Reference to this rectangle after subtraction.
	 */
	TIntRect& operator-=(const IntPointType& Point)
	{
		Min -= Point;
		Max -= Point;

		return *this;
	}

	/**
	 * Gets the result of scaling on this rectangle.
	 *
	 * @param Scale What to multiply this rectangle by.
	 * @return New scaled rectangle.
	 */
	[[nodiscard]] TIntRect operator*(IntType Scale) const
	{
		return TIntRect(Min * Scale, Max * Scale);
	}

	/**
	 * Gets the result of division on this rectangle.
	 *
	 * @param Div What to divide this rectangle by.
	 * @return New divided rectangle.
	 */
	[[nodiscard]] TIntRect operator/(IntType Div) const
	{
		return TIntRect(Min / Div, Max / Div);
	}

	/**
	 * Gets the result of adding a point to this rectangle.
	 *
	 * @param Point The point to add to both points in the rectangle.
	 * @return New rectangle with point added to it.
	 */
	[[nodiscard]] TIntRect operator+(const IntPointType& Point) const
	{
		return TIntRect(Min + Point, Max + Point);
	}

	/**
	 * Gets the result of dividing a point with this rectangle.
	 *
	 * @param Point The point to divide with.
	 * @return New rectangle with point divided.
	 */
	[[nodiscard]] TIntRect operator/(const IntPointType& Point) const
	{
		return TIntRect(Min / Point, Max / Point);
	}

	/**
	 * Gets the result of subtracting a point from this rectangle.
	 *
	 * @param Point The point to subtract from both points in the rectangle.
	 * @return New rectangle with point subtracted from it.
	 */
	[[nodiscard]] TIntRect operator-(const IntPointType& Point) const
	{
		return TIntRect(Min - Point, Max - Point);
	}

	/**
	 * Gets the result of adding two rectangles together.
	 *
	 * @param Other The other rectangle to add to this.
	 * @return New rectangle after both are added together.
	 */
	[[nodiscard]] TIntRect operator+(const TIntRect& Other) const
	{
		return TIntRect(Min + Other.Min, Max + Other.Max);
	}

	/**
	 * Gets the result of subtracting a rectangle from this one.
	 *
	 * @param Other The other rectangle to subtract from this.
	 * @return New rectangle after one is subtracted from this.
	 */
	[[nodiscard]] TIntRect operator-(const TIntRect& Other) const
	{
		return TIntRect(Min - Other.Min, Max - Other.Max);
	}

	/**
	 * Calculates the area of this rectangle.
	 *
	 * @return The area of this rectangle.
	 */
	[[nodiscard]] IntType Area() const
	{
		return (Max.X - Min.X) * (Max.Y - Min.Y);
	}

	/**
	 * Creates a rectangle from the bottom part of this rectangle.
	 *
	 * @param InHeight Height of the new rectangle (<= rectangles original height).
	 * @return The new rectangle.
	 */

	[[nodiscard]] TIntRect Bottom(IntType InHeight) const
	{
		return TIntRect(Min.X, FMath::Max(Min.Y, Max.Y - InHeight), Max.X, Max.Y);
	}

	/**
	 * Clip a rectangle using the bounds of another rectangle.
	 *
	 * @param Other The other rectangle to clip against.
	 */
	void Clip(const TIntRect& R)
	{
		Min.X = FMath::Max<IntType>(Min.X, R.Min.X);
		Min.Y = FMath::Max<IntType>(Min.Y, R.Min.Y);
		Max.X = FMath::Min<IntType>(Max.X, R.Max.X);
		Max.Y = FMath::Min<IntType>(Max.Y, R.Max.Y);

		// return zero area if not overlapping
		Max.X = FMath::Max<IntType>(Min.X, Max.X);
		Max.Y = FMath::Max<IntType>(Min.Y, Max.Y);
	}

	/** Combines the two rectangles. */
	void Union(const TIntRect& R)
	{
		Min.X = FMath::Min<IntType>(Min.X, R.Min.X);
		Min.Y = FMath::Min<IntType>(Min.Y, R.Min.Y);
		Max.X = FMath::Max<IntType>(Max.X, R.Max.X);
		Max.Y = FMath::Max<IntType>(Max.Y, R.Max.Y);
	}

	/**
	 * Returns true if the two rects have any overlap.
	 *
	 * @param Other The rect to compare with.
	 * @return true if the rectangle overlaps the other rectangle, false otherwise.
	 *
	 * @note  This function assumes rects have open bounds, i.e. rects with
	 *        coincident borders on any edge will not overlap.
	 */
	[[nodiscard]] bool Intersect(const TIntRect& Other) const
	{
		return Other.Min.X < Max.X && Other.Max.X > Min.X && Other.Min.Y < Max.Y && Other.Max.Y > Min.Y;
	}

	/**
	 * Test whether this rectangle contains a point.
	 *
	 * @param Point The point to test against.
	 * @return true if the rectangle contains the specified point, false otherwise.
	 *
	 * @note  This function assumes rects have half-open bounds, i.e. points are contained
	 *        by the minimum border of the box, but not the maximum border.
	 */
	[[nodiscard]] bool Contains(IntPointType P) const
	{
		return P.X >= Min.X && P.X < Max.X&& P.Y >= Min.Y && P.Y < Max.Y;
	}

	/**
	 * Test whether this rectangle fully contains other rectangle.
	 */
	[[nodiscard]] bool Contains(const FIntRect& Other) const
	{
		return Other.Min.X >= Min.X && Other.Max.X <= Max.X &&
			   Other.Min.Y >= Min.Y && Other.Max.Y <= Max.Y;
	}

	/**
	 * Gets the Center and Extents of this rectangle.
	 *
	 * @param OutCenter Will contain the center point.
	 * @param OutExtent Will contain the extent.
	 */
	void GetCenterAndExtents(IntPointType& OutCenter, IntPointType& OutExtent) const
	{
		OutExtent.X = (Max.X - Min.X) / 2;
		OutExtent.Y = (Max.Y - Min.Y) / 2;

		OutCenter.X = Min.X + OutExtent.X;
		OutCenter.Y = Min.Y + OutExtent.Y;
	}

	/**
	 * Gets the Height of the rectangle.
	 *
	 * @return The Height of the rectangle.
	 */
	[[nodiscard]] IntType Height() const
	{
		return (Max.Y - Min.Y);
	}


	/**
	 * Inflates or deflates the rectangle.
	 *
	 * @param Amount The amount to inflate or deflate the rectangle on each side.
	 */
	void InflateRect(IntType Amount)
	{
		Min.X -= Amount;
		Min.Y -= Amount;
		Max.X += Amount;
		Max.Y += Amount;
	}
	/**
	 * Adds to this rectangle to include a given point.
	 *
	 * @param Point The point to increase the rectangle to.
	 */
	void Include(IntPointType Point)
	{
		Min.X = FMath::Min(Min.X, Point.X);
		Min.Y = FMath::Min(Min.Y, Point.Y);
		Max.X = FMath::Max(Max.X, Point.X);
		Max.Y = FMath::Max(Max.Y, Point.Y);
	}

	/**
	 * Gets a new rectangle from the inner of this one.
	 *
	 * @param Shrink How much to remove from each point of this rectangle.
	 * @return New inner Rectangle.
	 */
	[[nodiscard]] TIntRect Inner(IntPointType Shrink) const
	{
		return TIntRect(Min + Shrink, Max - Shrink);
	}
	/**
	 * Creates a rectangle from the right hand side of this rectangle.
	 *
	 * @param InWidth Width of the new rectangle (<= rectangles original width).
	 * @return The new rectangle.
	 */
	[[nodiscard]] TIntRect Right(IntType InWidth) const
	{
		return TIntRect(FMath::Max(Min.X, Max.X - InWidth), Min.Y, Max.X, Max.Y);
	}

	/**
	 * Scales a rectangle using a floating point number.
	 *
	 * @param Fraction What to scale the rectangle by
	 * @return New scaled rectangle.
	 */
	[[nodiscard]] TIntRect Scale(double Fraction) const
	{
		using Vec2D = UE::Math::TVector2<double>;
		const Vec2D Min2D = Vec2D((double)Min.X, (double)Min.Y) * Fraction;
		const Vec2D Max2D = Vec2D((double)Max.X, (double)Max.Y) * Fraction;

		return TIntRect(
			IntCastChecked<IntType>(FMath::FloorToInt64(Min2D.X)),
			IntCastChecked<IntType>(FMath::FloorToInt64(Min2D.Y)),
			IntCastChecked<IntType>(FMath::CeilToInt64(Max2D.X)),
			IntCastChecked<IntType>(FMath::CeilToInt64(Max2D.Y))
		);
	}

	/**
	 * Gets the distance from one corner of the rectangle to the other.
	 *
	 * @return The distance from one corner of the rectangle to the other.
	 */
	[[nodiscard]] IntPointType Size() const
	{
		return IntPointType(Max.X - Min.X, Max.Y - Min.Y);
	}

	/**
	 * Get a textual representation of this rectangle.
	 *
	 * @return A string describing the rectangle.
	 */
	[[nodiscard]] FString ToString() const
	{
		return FString::Printf(TEXT("Min=(%s) Max=(%s)"), *Min.ToString(), *Max.ToString());
	}

	/**
	 * Gets the width of the rectangle.
	 *
	 * @return The width of the rectangle.
	 */
	[[nodiscard]] IntType Width() const
	{
		return Max.X - Min.X;
	}

	/**
	 * Returns true if the rectangle is 0 x 0.
	 *
	 * @return true if the rectangle is 0 x 0.
	 */
	[[nodiscard]] bool IsEmpty() const
	{
		return Width() == 0 && Height() == 0;
	}

	/**
	 * Divides a rectangle and rounds up to the nearest integer.
	 *
	 * @param lhs The Rectangle to divide.
	 * @param Div What to divide by.
	 * @return New divided rectangle.
	 */
	[[nodiscard]] static TIntRect DivideAndRoundUp(TIntRect lhs, IntType Div)
	{
		return DivideAndRoundUp(lhs, IntPointType(Div, Div));
	}

	[[nodiscard]] static TIntRect DivideAndRoundUp(TIntRect lhs, IntPointType Div)
	{
		return TIntRect(lhs.Min / Div, IntPointType::DivideAndRoundUp(lhs.Max, Div));
	}

	/**
	 * Gets number of points in the Rectangle.
	 *
	 * @return Number of points in the Rectangle.
	 */
	[[nodiscard]] static constexpr int32 Num()
	{
		return 2;
	}

public:

	/**
	 * Serializes the Rectangle.
	 *
	 * @param Ar The archive to serialize into.
	 * @param Rect The rectangle to serialize.
	 * @return Reference to the Archive after serialization.
	 */
	friend FArchive& operator<<( FArchive& Ar, TIntRect& Rect )
	{
		return Ar << Rect.Min.X << Rect.Min.Y << Rect.Max.X << Rect.Max.Y;
	}
};

template <typename IntType>
uint32 GetTypeHash(const TIntRect<IntType>& InRect)
{
	return HashCombine(GetTypeHash(InRect.Min), GetTypeHash(InRect.Max));
}

} //! namespace UE::Math

template <> struct TIsPODType<FInt32Rect> { enum { Value = true }; };
template <> struct TIsPODType<FUint32Rect> { enum { Value = true }; };

template <> struct TIsUECoreVariant<FInt32Rect>  { enum { Value = true }; };
template <> struct TIsUECoreVariant<FUint32Rect> { enum { Value = true }; };

template <> struct TIsPODType<FInt64Rect> { enum { Value = true }; };
template <> struct TIsPODType<FUint64Rect> { enum { Value = true }; };

template <> struct TIsUECoreVariant<FInt64Rect> { enum { Value = true }; };
template <> struct TIsUECoreVariant<FUint64Rect> { enum { Value = true }; };
