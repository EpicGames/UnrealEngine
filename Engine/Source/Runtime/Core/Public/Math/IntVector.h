// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Crc.h"
#include "Misc/Parse.h"
#include "Math/MathFwd.h" // IWYU pragma: export
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Serialization/StructuredArchive.h"
#include "Misc/LargeWorldCoordinatesSerializer.h"

namespace UE::Math
{

/**
 * Structure for integer vectors in 3-d space.
 */
template <typename InIntType>
struct TIntVector3
{
	using IntType = InIntType;
	static_assert(std::is_integral_v<IntType>, "Only an integer types are supported.");

	union
	{
		struct
		{
			/** Holds the vector's x-coordinate. */
			IntType X;

			/** Holds the vector's y-coordinate. */
			IntType Y;

			/** Holds the vector's z-coordinate. */
			IntType Z;
		};

		UE_DEPRECATED(all, "For internal use only")
		IntType XYZ[3];
	};

	/** An int vector with zeroed values. */
	static const TIntVector3 ZeroValue;

	/** An int vector with INDEX_NONE values. */
	static const TIntVector3 NoneValue;

	/**
	 * Default constructor (no initialization).
	 */
	[[nodiscard]] TIntVector3() = default;

	/**
	 * Creates and initializes a new instance with the specified coordinates.
	 *
	 * @param InX The x-coordinate.
	 * @param InY The y-coordinate.
	 * @param InZ The z-coordinate.
	 */
	[[nodiscard]] TIntVector3(IntType InX, IntType InY, IntType InZ)
		: X(InX)
		, Y(InY)
		, Z(InZ)
	{
	}

	/**
	 * Constructor
	 *
	 * @param InValue replicated to all components
	 */
	[[nodiscard]] explicit TIntVector3(IntType InValue)
		: X(InValue)
		, Y(InValue)
		, Z(InValue)
	{
	}

	[[nodiscard]] explicit TIntVector3(TIntVector4<IntType> Other)
		: X(Other.X)
		, Y(Other.Y)
		, Z(Other.Z)
	{
	}

	/**
	 * Constructor
	 *
	 * @param InVector float vector converted to int
	 */
	template <typename FloatType>
	[[nodiscard]] explicit TIntVector3(TVector<FloatType> InVector);

	/**
	 * Constructor
	 *
	 * @param EForceInit Force init enum
	 */
	[[nodiscard]] explicit TIntVector3(EForceInit)
		: X(0)
		, Y(0)
		, Z(0)
	{
	}

	/**
	 * Converts to another int type. Checks that the cast will succeed.
	 */
	template <typename OtherIntType>
	[[nodiscard]] explicit TIntVector3(TIntVector3<OtherIntType> Other)
		: X(IntCastChecked<IntType>(Other.X))
		, Y(IntCastChecked<IntType>(Other.Y))
		, Z(IntCastChecked<IntType>(Other.Z))
	{
	}

	// Workaround for clang deprecation warnings for deprecated XYZ member in implicitly-defined special member functions
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TIntVector3(TIntVector3&&) = default;
	TIntVector3(const TIntVector3&) = default;
	TIntVector3& operator=(TIntVector3&&) = default;
	TIntVector3& operator=(const TIntVector3&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Gets specific component of a vector.
	 *
	 * @param ComponentIndex Index of vector component.
	 * @return const reference to component.
	 */
	[[nodiscard]] const IntType& operator()(int32 ComponentIndex) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XYZ[ComponentIndex];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Gets specific component of a vector.
	 *
	 * @param ComponentIndex Index of vector component.
	 * @return reference to component.
	 */
	[[nodiscard]] IntType& operator()(int32 ComponentIndex)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XYZ[ComponentIndex];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Gets specific component of a vector.
	 *
	 * @param ComponentIndex Index of vector component.
	 * @return const reference to component.
	 */
	[[nodiscard]] const IntType& operator[](int32 ComponentIndex) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XYZ[ComponentIndex];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Gets specific component of a vector.
	 *
	 * @param ComponentIndex Index of vector component.
	 * @return reference to component.
	 */
	[[nodiscard]] IntType& operator[](int32 ComponentIndex)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XYZ[ComponentIndex];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Compares vectors for equality.
	 *
	 * @param Other The other int vector being compared.
	 * @return true if the vectors are equal, false otherwise..
	 */
	[[nodiscard]] bool operator==(const TIntVector3& Other) const
	{
		return X == Other.X && Y == Other.Y && Z == Other.Z;
	}

	/**
	 * Compares vectors for inequality.
	 *
	 * @param Other The other int vector being compared.
	 * @return true if the vectors are not equal, false otherwise..
	 */
	[[nodiscard]] bool operator!=(const TIntVector3& Other) const
	{
		return X != Other.X || Y != Other.Y || Z != Other.Z;
	}

	/**
	 * Multiplies this vector with another vector, using component-wise multiplication.
	 *
	 * @param Other The vector to multiply with.
	 * @return Reference to this vector after multiplication.
	 */
	TIntVector3& operator*=(const TIntVector3& Other)
	{
		X *= Other.X;
		Y *= Other.Y;
		Z *= Other.Z;

		return *this;
	}

	/**
	 * Scales this vector.
	 *
	 * @param Scale What to multiply the vector by.
	 * @return Reference to this vector after multiplication.
	 */
	TIntVector3& operator*=(IntType Scale)
	{
		X *= Scale;
		Y *= Scale;
		Z *= Scale;

		return *this;
	}

	/**
	 * Divides this vector.
	 *
	 * @param Divisor What to divide the vector by.
	 * @return Reference to this vector after division.
	 */
	TIntVector3& operator/=(IntType Divisor)
	{
		X /= Divisor;
		Y /= Divisor;
		Z /= Divisor;

		return *this;
	}

	/**
	 * Remainder of division of this vector.
	 *
	 * @param Divisor What to divide the vector by.
	 * @return Reference to this vector after remainder.
	 */
	TIntVector3& operator%=(IntType Divisor)
	{
		X %= Divisor;
		Y %= Divisor;
		Z %= Divisor;

		return *this;
	}

	/**
	 * Adds to this vector.
	 *
	 * @param Other The vector to add to this vector.
	 * @return Reference to this vector after addition.
	 */
	TIntVector3& operator+=(const TIntVector3& Other)
	{
		X += Other.X;
		Y += Other.Y;
		Z += Other.Z;

		return *this;
	}

	/**
	 * Subtracts from this vector.
	 *
	 * @param Other The vector to subtract from this vector.
	 * @return Reference to this vector after subtraction.
	 */
	TIntVector3& operator-=(const TIntVector3& Other)
	{
		X -= Other.X;
		Y -= Other.Y;
		Z -= Other.Z;

		return *this;
	}

	/**
	 * Gets the result of component-wise multiplication of this vector by another.
	 *
	 * @param Other The vector to multiply with.
	 * @return The result of multiplication.
	 */
	[[nodiscard]] TIntVector3 operator*(const TIntVector3& Other) const
	{
		return TIntVector3(*this) *= Other;
	}

	/**
	 * Gets the result of scaling on this vector.
	 *
	 * @param Scale What to multiply the vector by.
	 * @return A new scaled int vector.
	 */
	[[nodiscard]] TIntVector3 operator*(IntType Scale) const
	{
		return TIntVector3(*this) *= Scale;
	}

	/**
	 * Gets the result of division on this vector.
	 *
	 * @param Divisor What to divide the vector by.
	 * @return A new divided int vector.
	 */
	[[nodiscard]] TIntVector3 operator/(IntType Divisor) const
	{
		return TIntVector3(*this) /= Divisor;
	}

	/**
	 * Gets the remainder of division on this vector.
	 *
	 * @param Divisor What to divide the vector by.
	 * @return A new remainder int vector.
	 */
	[[nodiscard]] TIntVector3 operator%(IntType Divisor) const
	{
		return TIntVector3(*this) %= Divisor;
	}

	/**
	 * Gets the result of addition on this vector.
	 *
	 * @param Other The other vector to add to this.
	 * @return A new combined int vector.
	 */
	[[nodiscard]] TIntVector3 operator+(const TIntVector3& Other) const
	{
		return TIntVector3(*this) += Other;
	}

	/**
	 * Gets the result of subtraction from this vector.
	 *
	 * @param Other The other vector to subtract from this.
	 * @return A new subtracted int vector.
	 */
	[[nodiscard]] TIntVector3 operator-(const TIntVector3& Other) const
	{
		return TIntVector3(*this) -= Other;
	}

	/**
	 * Shifts all components to the right.
	 *
	 * @param Shift The number of bits to shift.
	 * @return A new shifted int vector.
	 */
	[[nodiscard]] TIntVector3 operator>>(IntType Shift) const
	{
		return TIntVector3(X >> Shift, Y >> Shift, Z >> Shift);
	}

	/**
	 * Shifts all components to the left.
	 *
	 * @param Shift The number of bits to shift.
	 * @return A new shifted int vector.
	 */
	[[nodiscard]] TIntVector3 operator<<(IntType Shift) const
	{
		return TIntVector3(X << Shift, Y << Shift, Z << Shift);
	}

	/**
	 * Component-wise AND.
	 *
	 * @param Value Number to AND with the each component.
	 * @return A new shifted int vector.
	 */
	[[nodiscard]] TIntVector3 operator&(IntType Value) const
	{
		return TIntVector3(X & Value, Y & Value, Z & Value);
	}

	/**
	 * Component-wise OR.
	 *
	 * @param Value Number to OR with the each component.
	 * @return A new shifted int vector.
	 */
	[[nodiscard]] TIntVector3 operator|(IntType Value) const
	{
		return TIntVector3(X | Value, Y | Value, Z | Value);
	}

	/**
	 * Component-wise XOR.
	 *
	 * @param Value Number to XOR with the each component.
	 * @return A new shifted int vector.
	 */
	[[nodiscard]] TIntVector3 operator^(IntType Value) const
	{
		return TIntVector3(X ^ Value, Y ^ Value, Z ^ Value);
	}

	/**
	 * Is vector equal to zero.
	 * @return is zero
	*/
	[[nodiscard]] bool IsZero() const
	{
		return *this == ZeroValue;
	}

	/**
	 * Gets the maximum value in the vector.
	 *
	 * @return The maximum value in the vector.
	 */
	[[nodiscard]] IntType GetMax() const
	{
		return FMath::Max(FMath::Max(X, Y), Z);
	}

	/**
	 * Get the maximum absolute value in the vector.
	 *
	 * @return The maximum absolute value in the vector.
	 */
	[[nodiscard]] IntType GetAbsMax() const
	{
		if constexpr (std::is_signed_v<IntType>)
		{
			return FMath::Max(FMath::Max(FMath::Abs(X), FMath::Abs(Y)), FMath::Abs(Z));
		}
		else
		{
			return GetMax();
		}
	}

	/**
	 * Gets the minimum value in the vector.
	 *
	 * @return The minimum value in the vector.
	 */
	[[nodiscard]] IntType GetMin() const
	{
		return FMath::Min(FMath::Min(X, Y), Z);
	}

	/**
	 * Get the minimum absolute value in the vector.
	 *
	 * @return The minimum absolute value in the vector.
	 */
	[[nodiscard]] IntType GetAbsMin() const
	{
		if constexpr (std::is_signed_v<IntType>)
		{
			return FMath::Min(FMath::Min(FMath::Abs(X), FMath::Abs(Y)), FMath::Abs(Z));
		}
		else
		{
			return GetMin();
		}
	}

	/**
	 * Get the component-wise max of this vector and the parameter vector.
	 * 
	 * @param Other The other vector to compare against.
	 * @return A vector where each component is the maximum value of the corresponding components of the two vectors.
	 */
	[[nodiscard]] TIntVector3 ComponentMax(const TIntVector3& Other) const
	{
		return TIntVector3(
			FMath::Max(X, Other.X),
			FMath::Max(Y, Other.Y),
			FMath::Max(Z, Other.Z));
	}

	/**
	 * Get the component-wise min of this vector and the parameter vector.
	 * 
	 * @param Other The other vector to compare against.
	 * @return A vector where each component is the minimum value of the corresponding components of the two vectors.
	 */
	[[nodiscard]] TIntVector3 ComponentMin(const TIntVector3& Other) const
	{
		return TIntVector3(
			FMath::Min(X, Other.X),
			FMath::Min(Y, Other.Y),
			FMath::Min(Z, Other.Z));
	}

	/**
	 * Gets the distance of this vector from (0,0,0).
	 *
	 * @return The distance of this vector from (0,0,0).
	 */
	[[nodiscard]] IntType Size() const
	{
		int64 LocalX64 = (int64)X;
		int64 LocalY64 = (int64)Y;
		int64 LocalZ64 = (int64)Z;
		return IntType(FMath::Sqrt(double(LocalX64 * LocalX64 + LocalY64 * LocalY64 + LocalZ64 * LocalZ64)));
	}

	/**
	 * Appends a textual representation of this vector to the output string builder.
	 *
	 * @param Out The string builder to append to.
	 */
	template <typename CharType>
	void AppendString(TStringBuilderBase<CharType>& Out) const
	{
		Out << "X=" << X << " Y=" << Y << " Z=" << Z;
	}

	/**
	 * Appends a textual representation of the parameter vector to the output string builder.
	 *
	 * @param Builder The string builder to append to.
	 * @param Vector The vector to append.
	 */
	template <typename CharType>
	friend TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const TIntVector3& Vector)
	{
		Vector.AppendString(Builder);
		return Builder;
	}

	/**
	 * Appends a textual representation of this vector to the output string.
	 *
	 * @param Out The string to append to.
	 */
	void AppendString(FString& Out) const
	{
		TStringBuilder<128> Builder;
		Builder << *this;
		Out.Append(Builder);
	}

	/**
	 * Get a textual representation of this vector.
	 *
	 * @return A string describing the vector.
	 */
	[[nodiscard]] FString ToString() const
	{
		FString Out;
		AppendString(Out);
		return Out;
	}

	/**
	 * Initialize this vector based on an FString. The String is expected to contain X=, Y=, Z=
	 * The vector will be bogus when InitFromString returns false.
	 *
	 * @param InSourceString FString containing the color values.
	 * @return true if the X,Y,Z values were read successfully; false otherwise.
	 */
	bool InitFromString(const FString& InSourceString)
	{
		X = Y = Z = 0;

		// The initialization is only successful if the X, Y and Z values can all be parsed from the string
		const bool bSuccessful = FParse::Value(*InSourceString, TEXT("X="), X) && FParse::Value(*InSourceString, TEXT("Y="), Y) && FParse::Value(*InSourceString, TEXT("Z="), Z);

		return bSuccessful;
	}

	/**
	 * Divide an int vector and round up the result.
	 *
	 * @param Lhs The int vector being divided.
	 * @param Divisor What to divide the int vector by.
	 * @return A new divided int vector.
	 */
	[[nodiscard]] static TIntVector3 DivideAndRoundUp(TIntVector3 Lhs, IntType Divisor)
	{
		return TIntVector3(FMath::DivideAndRoundUp(Lhs.X, Divisor), FMath::DivideAndRoundUp(Lhs.Y, Divisor), FMath::DivideAndRoundUp(Lhs.Z, Divisor));
	}

	[[nodiscard]] static TIntVector3 DivideAndRoundUp(TIntVector3 Lhs, TIntVector3 Divisor)
	{
		return TIntVector3(FMath::DivideAndRoundUp(Lhs.X, Divisor.X), FMath::DivideAndRoundUp(Lhs.Y, Divisor.Y), FMath::DivideAndRoundUp(Lhs.Z, Divisor.Z));
	}

	/**
	 * Gets the number of components a vector has.
	 *
	 * @return Number of components vector has.
	 */
	[[nodiscard]] static int32 Num()
	{
		return 3;
	}

	/**
	 * Serializes the Vector3.
	 *
	 * @param Ar The archive to serialize into.
	 * @param Vector The vector to serialize.
	 * @return Reference to the Archive after serialization.
	 */
	friend FArchive& operator<<(FArchive& Ar, TIntVector3& Vector)
	{
		return Ar << Vector.X << Vector.Y << Vector.Z;
	}

	friend void operator<<(FStructuredArchive::FSlot Slot, TIntVector3& Vector)
	{
		FStructuredArchive::FRecord Record = Slot.EnterRecord();
		Record << SA_VALUE(TEXT("X"), Vector.X);
		Record << SA_VALUE(TEXT("Y"), Vector.Y);
		Record << SA_VALUE(TEXT("Z"), Vector.Z);
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		if constexpr (std::is_same_v<IntType, int32>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, IntVector, Int32Vector, Int64Vector);
		}
		else if constexpr (std::is_same_v<IntType, int64>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, IntVector, Int64Vector, Int32Vector);
		}
		else if constexpr (std::is_same_v<IntType, uint32>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, UintVector, Uint32Vector, Uint64Vector);
		}
		else if constexpr (std::is_same_v<IntType, uint64>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, UintVector, Uint64Vector, Uint32Vector);
		}
		else
		{
			static_assert(sizeof(IntType) == 0, "Unimplemented");
			return false;
		}
	}
};

template <typename IntType>
const TIntVector3<IntType> TIntVector3<IntType>::ZeroValue(0, 0, 0);

template <typename IntType>
const TIntVector3<IntType> TIntVector3<IntType>::NoneValue(INDEX_NONE, INDEX_NONE, INDEX_NONE);

///////////////////////////////////////////////////////////////////////////////////////////////////

template <typename InIntType>
struct TIntVector2
{
	using IntType = InIntType;
	static_assert(std::is_integral_v<IntType>, "Only an integer types are supported.");

	union
	{
		struct
		{
			/** Holds the vector's x-coordinate. */
			IntType X;

			/** Holds the vector's y-coordinate. */
			IntType Y;
		};

		UE_DEPRECATED(all, "For internal use only")
		IntType XY[2];
	};

	/** An int vector with zeroed values. */
	static const TIntVector2 ZeroValue;

	/** An int vector with INDEX_NONE values. */
	static const TIntVector2 NoneValue;

	TIntVector2() = default;

	TIntVector2(IntType InX, IntType InY)
		: X(InX)
		, Y(InY)
	{
	}

	explicit TIntVector2(IntType InValue)
		: X(InValue)
		, Y(InValue)
	{
	}

	TIntVector2(EForceInit)
		: X(0)
		, Y(0)
	{
	}

	TIntVector2(TIntPoint<IntType> Other)
		: X(Other.X)
		, Y(Other.Y)
	{
	}

	explicit TIntVector2(TIntVector3<IntType> Other)
		: X(Other.X)
		, Y(Other.Y)
	{
	}

	/**
	 * Converts to another int type. Checks that the cast will succeed.
	 */
	template <typename OtherIntType>
	explicit TIntVector2(TIntVector2<OtherIntType> Other)
		: X(IntCastChecked<IntType>(Other.X))
		, Y(IntCastChecked<IntType>(Other.Y))
	{
	}

	// Workaround for clang deprecation warnings for deprecated XY member in implicitly-defined special member functions
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TIntVector2(TIntVector2&&) = default;
	TIntVector2(const TIntVector2&) = default;
	TIntVector2& operator=(TIntVector2&&) = default;
	TIntVector2& operator=(const TIntVector2&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	const IntType& operator[](int32 ComponentIndex) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XY[ComponentIndex];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	IntType& operator[](int32 ComponentIndex)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XY[ComponentIndex];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	bool operator==(const TIntVector2& Other) const
	{
		return X==Other.X && Y==Other.Y;
	}

	bool operator!=(const TIntVector2& Other) const
	{
		return X!=Other.X || Y!=Other.Y;
	}

	/**
	 * Multiplies this vector with another vector, using component-wise multiplication.
	 *
	 * @param Other The vector to multiply with.
	 * @return Reference to this vector after multiplication.
	 */
	TIntVector2& operator*=(const TIntVector2& Other)
	{
		X *= Other.X;
		Y *= Other.Y;

		return *this;
	}

	/**
	 * Scales this vector.
	 *
	 * @param Scale What to multiply the vector by.
	 * @return Reference to this vector after multiplication.
	 */
	TIntVector2& operator*=(IntType Scale)
	{
		X *= Scale;
		Y *= Scale;

		return *this;
	}

	/**
	 * Divides this vector.
	 *
	 * @param Divisor What to divide the vector by.
	 * @return Reference to this vector after division.
	 */
	TIntVector2& operator/=(IntType Divisor)
	{
		X /= Divisor;
		Y /= Divisor;

		return *this;
	}

	/**
	 * Remainder of division of this vector.
	 *
	 * @param Divisor What to divide the vector by.
	 * @return Reference to this vector after remainder.
	 */
	TIntVector2& operator%=(IntType Divisor)
	{
		X %= Divisor;
		Y %= Divisor;

		return *this;
	}

	/**
	 * Adds to this vector.
	 *
	 * @param Other The vector to add to this vector.
	 * @return Reference to this vector after addition.
	 */
	TIntVector2& operator+=(const TIntVector2& Other)
	{
		X += Other.X;
		Y += Other.Y;

		return *this;
	}

	/**
	 * Subtracts from this vector.
	 *
	 * @param Other The vector to subtract from this vector.
	 * @return Reference to this vector after subtraction.
	 */
	TIntVector2& operator-=(const TIntVector2& Other)
	{
		X -= Other.X;
		Y -= Other.Y;

		return *this;
	}

	/**
	 * Gets the result of component-wise multiplication of this vector by another.
	 *
	 * @param Other The vector to multiply with.
	 * @return The result of multiplication.
	 */
	TIntVector2 operator*(const TIntVector2& Other) const
	{
		return TIntVector2(*this) *= Other;
	}

	/**
	 * Gets the result of scaling on this vector.
	 *
	 * @param Scale What to multiply the vector by.
	 * @return A new scaled int vector.
	 */
	TIntVector2 operator*(IntType Scale) const
	{
		return TIntVector2(*this) *= Scale;
	}

	/**
	 * Gets the result of division on this vector.
	 *
	 * @param Divisor What to divide the vector by.
	 * @return A new divided int vector.
	 */
	TIntVector2 operator/(IntType Divisor) const
	{
		return TIntVector2(*this) /= Divisor;
	}

	/**
	 * Gets the remainder of division on this vector.
	 *
	 * @param Divisor What to divide the vector by.
	 * @return A new remainder int vector.
	 */
	TIntVector2 operator%(IntType Divisor) const
	{
		return TIntVector2(*this) %= Divisor;
	}

	/**
	 * Gets the result of addition on this vector.
	 *
	 * @param Other The other vector to add to this.
	 * @return A new combined int vector.
	 */
	TIntVector2 operator+(const TIntVector2& Other) const
	{
		return TIntVector2(*this) += Other;
	}

	/**
	 * Gets the result of subtraction from this vector.
	 *
	 * @param Other The other vector to subtract from this.
	 * @return A new subtracted int vector.
	 */
	TIntVector2 operator-(const TIntVector2& Other) const
	{
		return TIntVector2(*this) -= Other;
	}

	/**
	 * Shifts all components to the right.
	 *
	 * @param Shift The number of bits to shift.
	 * @return A new shifted int vector.
	 */
	TIntVector2 operator>>(IntType Shift) const
	{
		return TIntVector2(X >> Shift, Y >> Shift);
	}

	/**
	 * Shifts all components to the left.
	 *
	 * @param Shift The number of bits to shift.
	 * @return A new shifted int vector.
	 */
	TIntVector2 operator<<(IntType Shift) const
	{
		return TIntVector2(X << Shift, Y << Shift);
	}

	/**
	 * Component-wise AND.
	 *
	 * @param Value Number to AND with the each component.
	 * @return A new shifted int vector.
	 */
	TIntVector2 operator&(IntType Value) const
	{
		return TIntVector2(X & Value, Y & Value);
	}

	/**
	 * Component-wise OR.
	 *
	 * @param Value Number to OR with the each component.
	 * @return A new shifted int vector.
	 */
	TIntVector2 operator|(IntType Value) const
	{
		return TIntVector2(X | Value, Y | Value);
	}

	/**
	 * Component-wise XOR.
	 *
	 * @param Value Number to XOR with the each component.
	 * @return A new shifted int vector.
	 */
	TIntVector2 operator^(IntType Value) const
	{
		return TIntVector2(X ^ Value, Y ^ Value);
	}

	/**
	 * Is vector equal to zero.
	 * @return is zero
	*/
	bool IsZero() const
	{
		return *this == ZeroValue;
	}

	/**
	 * Gets the maximum value in the vector.
	 *
	 * @return The maximum value in the vector.
	 */
	IntType GetMax() const
	{
		return FMath::Max(X, Y);
	}

	/**
	 * Get the maximum absolute value in the vector.
	 *
	 * @return The maximum absolute value in the vector.
	 */
	IntType GetAbsMax() const
	{
		if constexpr (std::is_signed_v<IntType>)
		{
			return FMath::Max(FMath::Abs(X), FMath::Abs(Y));
		}
		else
		{
			return GetMax();
		}
	}

	/**
	 * Gets the minimum value in the vector.
	 *
	 * @return The minimum value in the vector.
	 */
	IntType GetMin() const
	{
		return FMath::Min(X, Y);
	}

	/**
	 * Get the minimum absolute value in the vector.
	 *
	 * @return The minimum absolute value in the vector.
	 */
	IntType GetAbsMin() const
	{
		if constexpr (std::is_signed_v<IntType>)
		{
			return FMath::Min(FMath::Abs(X), FMath::Abs(Y));
		}
		else
		{
			return GetMin();
		}
	}

	/**
	 * Get the component-wise max of this vector and the parameter vector.
	 * 
	 * @param Other The other vector to compare against.
	 * @return A vector where each component is the maximum value of the corresponding components of the two vectors.
	 */
	TIntVector2 ComponentMax(const TIntVector2& Other) const
	{
		return TIntVector2(
			FMath::Max(X, Other.X),
			FMath::Max(Y, Other.Y));
	}

	/**
	 * Get the component-wise min of this vector and the parameter vector.
	 * 
	 * @param Other The other vector to compare against.
	 * @return A vector where each component is the minimum value of the corresponding components of the two vectors.
	 */
	TIntVector2 ComponentMin(const TIntVector2& Other) const
	{
		return TIntVector2(
			FMath::Min(X, Other.X),
			FMath::Min(Y, Other.Y));
	}

	/**
	 * Appends a textual representation of this vector to the output string builder.
	 *
	 * @param Out The string builder to append to.
	 */
	template <typename CharType>
	void AppendString(TStringBuilderBase<CharType>& Out) const
	{
		Out << "X=" << X << " Y=" << Y;
	}

	/**
	 * Appends a textual representation of the parameter vector to the output string builder.
	 *
	 * @param Builder The string builder to append to.
	 * @param Vector The vector to append.
	 */
	template <typename CharType>
	friend TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const TIntVector2& Vector)
	{
		Vector.AppendString(Builder);
		return Builder;
	}

	/**
	 * Appends a textual representation of this vector to the output string.
	 *
	 * @param Out The string to append to.
	 */
	void AppendString(FString& Out) const
	{
		TStringBuilder<128> Builder;
		Builder << *this;
		Out.Append(Builder);
	}

	/**
	 * Get a textual representation of this vector.
	 *
	 * @return A string describing the vector.
	 */
	FString ToString() const
	{
		FString Out;
		AppendString(Out);
		return Out;
	}

	/**
	 * Initialize this FIntVector based on an FString. The String is expected to contain X=, Y=
	 * The FIntVector will be bogus when InitFromString returns false.
	 *
	 * @param InSourceString FString containing the color values.
	 * @return true if the X,Y values were read successfully; false otherwise.
	 */
	bool InitFromString(const FString& InSourceString)
	{
		X = Y = 0;

		// The initialization is only successful if the X and Y values can all be parsed from the string
		const bool bSuccessful = FParse::Value(*InSourceString, TEXT("X="), X) && FParse::Value(*InSourceString, TEXT("Y="), Y);

		return bSuccessful;
	}

	/**
	 * Divide an int vector and round up the result.
	 *
	 * @param Lhs The int vector being divided.
	 * @param Divisor What to divide the int vector by.
	 * @return A new divided int vector.
	 */
	static TIntVector2 DivideAndRoundUp(TIntVector2 Lhs, IntType Divisor)
	{
		return TIntVector2(FMath::DivideAndRoundUp(Lhs.X, Divisor), FMath::DivideAndRoundUp(Lhs.Y, Divisor));
	}

	static TIntVector2 DivideAndRoundUp(TIntVector2 Lhs, TIntVector2 Divisor)
	{
		return TIntVector2(FMath::DivideAndRoundUp(Lhs.X, Divisor.X), FMath::DivideAndRoundUp(Lhs.Y, Divisor.Y));
	}

	/**
	 * Gets the number of components a vector has.
	 *
	 * @return Number of components vector has.
	 */
	static int32 Num()
	{
		return 2;
	}

	/**
	 * Serializes the Vector2.
	 *
	 * @param Ar The archive to serialize into.
	 * @param Vector The vector to serialize.
	 * @return Reference to the Archive after serialization.
	 */
	friend FArchive& operator<<(FArchive& Ar, TIntVector2& Vector)
	{
		return Ar << Vector.X << Vector.Y;
	}

	friend void operator<<(FStructuredArchive::FSlot Slot, TIntVector2& Vector)
	{
		FStructuredArchive::FRecord Record = Slot.EnterRecord();
		Record << SA_VALUE(TEXT("X"), Vector.X);
		Record << SA_VALUE(TEXT("Y"), Vector.Y);
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		if constexpr (std::is_same_v<IntType, int32>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, IntVector2, Int32Vector2, Int64Vector2);
		}
		else if constexpr (std::is_same_v<IntType, int64>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, IntVector2, Int64Vector2, Int32Vector2);
		}
		else if constexpr (std::is_same_v<IntType, uint32>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, UintVector2, Uint32Vector2, Uint64Vector2);
		}
		else if constexpr (std::is_same_v<IntType, uint64>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, UintVector2, Uint64Vector2, Uint32Vector2);
		}
		else
		{
			static_assert(sizeof(IntType) == 0, "Unimplemented");
			return false;
		}
	}
};

template <typename IntType>
const TIntVector2<IntType> TIntVector2<IntType>::ZeroValue(0, 0);

template <typename IntType>
const TIntVector2<IntType> TIntVector2<IntType>::NoneValue(INDEX_NONE, INDEX_NONE);

///////////////////////////////////////////////////////////////////////////////////////////////////

template <typename InIntType>
struct TIntVector4
{
	using IntType = InIntType;
	static_assert(std::is_integral_v<IntType>, "Only an integer types are supported.");

	union
	{
		struct
		{
			/** Holds the vector's x-coordinate. */
			IntType X;

			/** Holds the vector's y-coordinate. */
			IntType Y;

			/** Holds the vector's z-coordinate. */
			IntType Z;
			
			/** Holds the vector's w-coordinate. */
			IntType W;
		};

		UE_DEPRECATED(all, "For internal use only")
		IntType XYZW[4];
	};

	/** An int vector with zeroed values. */
	static const TIntVector4 ZeroValue;

	/** An int vector with INDEX_NONE values. */
	static const TIntVector4 NoneValue;

	TIntVector4() = default;

	TIntVector4(IntType InX, IntType InY, IntType InZ, IntType InW)
		: X(InX)
		, Y(InY)
		, Z(InZ)
		, W(InW)
	{
	}

	explicit TIntVector4(IntType InValue)
		: X(InValue)
		, Y(InValue)
		, Z(InValue)
		, W(InValue)
	{
	}

	explicit TIntVector4(const TIntVector3<IntType>& InValue, IntType InW = 0)
		: X(InValue.X)
		, Y(InValue.Y)
		, Z(InValue.Z)
		, W(InW)
	{
	}

	TIntVector4(EForceInit)
		: X(0)
		, Y(0)
		, Z(0)
		, W(0)
	{
	}

	/**
	 * Converts to another int type. Checks that the cast will succeed.
	 */
	template <typename OtherIntType>
	explicit TIntVector4(TIntVector4<OtherIntType> Other)
		: X(IntCastChecked<IntType>(Other.X))
		, Y(IntCastChecked<IntType>(Other.Y))
		, Z(IntCastChecked<IntType>(Other.Z))
		, W(IntCastChecked<IntType>(Other.W))
	{
	}

	// Workaround for clang deprecation warnings for deprecated XYZW member in implicitly-defined special member functions
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TIntVector4(TIntVector4&&) = default;
	TIntVector4(const TIntVector4&) = default;
	TIntVector4& operator=(TIntVector4&&) = default;
	TIntVector4& operator=(const TIntVector4&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Gets specific component of a vector.
	 *
	 * @param ComponentIndex Index of vector component.
	 * @return const reference to component.
	 */
	const IntType& operator()(int32 ComponentIndex) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XYZW[ComponentIndex];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Gets specific component of a vector.
	 *
	 * @param ComponentIndex Index of vector component.
	 * @return reference to component.
	 */
	IntType& operator()(int32 ComponentIndex)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XYZW[ComponentIndex];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Gets specific component of a vector.
	 *
	 * @param ComponentIndex Index of vector component.
	 * @return const reference to component.
	 */
	const IntType& operator[](int32 ComponentIndex) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XYZW[ComponentIndex];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Gets specific component of a vector.
	 *
	 * @param ComponentIndex Index of vector component.
	 * @return reference to component.
	 */
	IntType& operator[](int32 ComponentIndex)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XYZW[ComponentIndex];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Compares vectors for equality.
	 *
	 * @param Other The other int vector being compared.
	 * @return true if the vectors are equal, false otherwise..
	 */
	bool operator==(const TIntVector4& Other) const
	{
		return X == Other.X && Y == Other.Y && Z == Other.Z && W == Other.W;
	}

	/**
	 * Compares vectors for inequality.
	 *
	 * @param Other The other int vector being compared.
	 * @return true if the vectors are not equal, false otherwise..
	 */
	bool operator!=(const TIntVector4& Other) const
	{
		return X != Other.X || Y != Other.Y || Z != Other.Z || W != Other.W;
	}

	/**
	 * Multiplies this vector with another vector, using component-wise multiplication.
	 *
	 * @param Other The vector to multiply with.
	 * @return Reference to this vector after multiplication.
	 */
	TIntVector4& operator*=(const TIntVector4& Other)
	{
		X *= Other.X;
		Y *= Other.Y;
		Z *= Other.Z;
		W *= Other.W;

		return *this;
	}

	/**
	 * Scales this vector.
	 *
	 * @param Scale What to multiply the vector by.
	 * @return Reference to this vector after multiplication.
	 */
	TIntVector4& operator*=(IntType Scale)
	{
		X *= Scale;
		Y *= Scale;
		Z *= Scale;
		W *= Scale;

		return *this;
	}

	/**
	 * Divides this vector.
	 *
	 * @param Divisor What to divide the vector by.
	 * @return Reference to this vector after division.
	 */
	TIntVector4& operator/=(IntType Divisor)
	{
		X /= Divisor;
		Y /= Divisor;
		Z /= Divisor;
		W /= Divisor;

		return *this;
	}

	/**
	 * Remainder of division of this vector.
	 *
	 * @param Divisor What to divide the vector by.
	 * @return Reference to this vector after remainder.
	 */
	TIntVector4& operator%=(IntType Divisor)
	{
		X %= Divisor;
		Y %= Divisor;
		Z %= Divisor;
		W %= Divisor;

		return *this;
	}

	/**
	 * Adds to this vector.
	 *
	 * @param Other The vector to add to this vector.
	 * @return Reference to this vector after addition.
	 */
	TIntVector4& operator+=(const TIntVector4& Other)
	{
		X += Other.X;
		Y += Other.Y;
		Z += Other.Z;
		W += Other.W;

		return *this;
	}

	/**
	 * Subtracts from this vector.
	 *
	 * @param Other The vector to subtract from this vector.
	 * @return Reference to this vector after subtraction.
	 */
	TIntVector4& operator-=(const TIntVector4& Other)
	{
		X -= Other.X;
		Y -= Other.Y;
		Z -= Other.Z;
		W -= Other.W;

		return *this;
	}

	/**
	 * Gets the result of component-wise multiplication of this vector by another.
	 *
	 * @param Other The vector to multiply with.
	 * @return The result of multiplication.
	 */
	TIntVector4 operator*(const TIntVector4& Other) const
	{
		return TIntVector4(*this) *= Other;
	}

	/**
	 * Gets the result of scaling on this vector.
	 *
	 * @param Scale What to multiply the vector by.
	 * @return A new scaled int vector.
	 */
	TIntVector4 operator*(IntType Scale) const
	{
		return TIntVector4(*this) *= Scale;
	}

	/**
	 * Gets the result of division on this vector.
	 *
	 * @param Divisor What to divide the vector by.
	 * @return A new divided int vector.
	 */
	TIntVector4 operator/(IntType Divisor) const
	{
		return TIntVector4(*this) /= Divisor;
	}

	/**
	 * Gets the remainder of division on this vector.
	 *
	 * @param Divisor What to divide the vector by.
	 * @return A new remainder int vector.
	 */
	TIntVector4 operator%(IntType Divisor) const
	{
		return TIntVector4(*this) %= Divisor;
	}

	/**
	 * Gets the result of addition on this vector.
	 *
	 * @param Other The other vector to add to this.
	 * @return A new combined int vector.
	 */
	TIntVector4 operator+(const TIntVector4& Other) const
	{
		return TIntVector4(*this) += Other;
	}

	/**
	 * Gets the result of subtraction from this vector.
	 *
	 * @param Other The other vector to subtract from this.
	 * @return A new subtracted int vector.
	 */
	TIntVector4 operator-(const TIntVector4& Other) const
	{
		return TIntVector4(*this) -= Other;
	}

	/**
	 * Shifts all components to the right.
	 *
	 * @param Shift The number of bits to shift.
	 * @return A new shifted int vector.
	 */
	TIntVector4 operator>>(IntType Shift) const
	{
		return TIntVector4(X >> Shift, Y >> Shift, Z >> Shift, W >> Shift);
	}

	/**
	 * Shifts all components to the left.
	 *
	 * @param Shift The number of bits to shift.
	 * @return A new shifted int vector.
	 */
	TIntVector4 operator<<(IntType Shift) const
	{
		return TIntVector4(X << Shift, Y << Shift, Z << Shift, W << Shift);
	}

	/**
	 * Component-wise AND.
	 *
	 * @param Value Number to AND with the each component.
	 * @return A new shifted int vector.
	 */
	TIntVector4 operator&(IntType Value) const
	{
		return TIntVector4(X & Value, Y & Value, Z & Value, W & Value);
	}

	/**
	 * Component-wise OR.
	 *
	 * @param Value Number to OR with the each component.
	 * @return A new shifted int vector.
	 */
	TIntVector4 operator|(IntType Value) const
	{
		return TIntVector4(X | Value, Y | Value, Z | Value, W | Value);
	}

	/**
	 * Component-wise XOR.
	 *
	 * @param Value Number to XOR with the each component.
	 * @return A new shifted int vector.
	 */
	TIntVector4 operator^(IntType Value) const
	{
		return TIntVector4(X ^ Value, Y ^ Value, Z ^ Value, W ^ Value);
	}

	/**
	 * Is vector equal to zero.
	 * @return is zero
	*/
	bool IsZero() const
	{
		return *this == ZeroValue;
	}

	/**
	 * Gets the maximum value in the vector.
	 *
	 * @return The maximum value in the vector.
	 */
	IntType GetMax() const
	{
		return FMath::Max(FMath::Max(FMath::Max(X, Y), Z), W);
	}

	/**
	 * Get the maximum absolute value in the vector.
	 *
	 * @return The maximum absolute value in the vector.
	 */
	IntType GetAbsMax() const
	{
		if constexpr (std::is_signed_v<IntType>)
		{
			return FMath::Max(FMath::Max(FMath::Max(FMath::Abs(X), FMath::Abs(Y)), FMath::Abs(Z)), FMath::Abs(W));
		}
		else
		{
			return GetMax();
		}
	}

	/**
	 * Gets the minimum value in the vector.
	 *
	 * @return The minimum value in the vector.
	 */
	IntType GetMin() const
	{
		return FMath::Min(FMath::Min(FMath::Min(X, Y), Z), W);
	}

	/**
	 * Get the minimum absolute value in the vector.
	 *
	 * @return The minimum absolute value in the vector.
	 */
	IntType GetAbsMin() const
	{
		if constexpr (std::is_signed_v<IntType>)
		{
			return FMath::Min(FMath::Min(FMath::Min(FMath::Abs(X), FMath::Abs(Y)), FMath::Abs(Z)), FMath::Abs(W));
		}
		else
		{
			return GetMin();
		}
	}

	/**
	 * Get the component-wise max of this vector and the parameter vector.
	 * 
	 * @param Other The other vector to compare against.
	 * @return A vector where each component is the maximum value of the corresponding components of the two vectors.
	 */
	TIntVector4 ComponentMax(const TIntVector4& Other) const
	{
		return TIntVector4(
			FMath::Max(X, Other.X),
			FMath::Max(Y, Other.Y),
			FMath::Max(Z, Other.Z),
			FMath::Max(W, Other.W));
	}

	/**
	 * Get the component-wise min of this vector and the parameter vector.
	 * 
	 * @param Other The other vector to compare against.
	 * @return A vector where each component is the minimum value of the corresponding components of the two vectors.
	 */
	TIntVector4 ComponentMin(const TIntVector4& Other) const
	{
		return TIntVector4(
			FMath::Min(X, Other.X),
			FMath::Min(Y, Other.Y),
			FMath::Min(Z, Other.Z),
			FMath::Min(W, Other.W));
	}

	/**
	 * Appends a textual representation of this vector to the output string builder.
	 *
	 * @param Out The string builder to append to.
	 */
	template <typename CharType>
	void AppendString(TStringBuilderBase<CharType>& Out) const
	{
		Out << "X=" << X << " Y=" << Y << " Z=" << Z << " W=" << W;
	}

	/**
	 * Appends a textual representation of the parameter vector to the output string builder.
	 *
	 * @param Builder The string builder to append to.
	 * @param Vector The vector to append.
	 */
	template <typename CharType>
	friend TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const TIntVector4& Vector)
	{
		Vector.AppendString(Builder);
		return Builder;
	}

	/**
	 * Appends a textual representation of this vector to the output string.
	 *
	 * @param Out The string to append to.
	 */
	void AppendString(FString& Out) const
	{
		TStringBuilder<128> Builder;
		Builder << *this;
		Out.Append(Builder);
	}

	/**
	 * Get a textual representation of this vector.
	 *
	 * @return A string describing the vector.
	 */
	FString ToString() const
	{
		FString Out;
		AppendString(Out);
		return Out;
	}

	/**
	 * Initialize this vector based on an FString. The String is expected to contain X=, Y=, Z=, W=
	 * The vector will be bogus when InitFromString returns false.
	 *
	 * @param InSourceString FString containing the color values.
	 * @return true if the X,Y,Z,W values were read successfully; false otherwise.
	 */
	bool InitFromString(const FString& InSourceString)
	{
		X = Y = Z = W = 0;

		// The initialization is only successful if the X, Y, Z and W values can all be parsed from the string
		const bool bSuccessful = FParse::Value(*InSourceString, TEXT("X="), X) && FParse::Value(*InSourceString, TEXT("Y="), Y) && FParse::Value(*InSourceString, TEXT("Z="), Z) && FParse::Value(*InSourceString, TEXT("W="), W);

		return bSuccessful;
	}

	/**
	 * Divide an int vector and round up the result.
	 *
	 * @param Lhs The int vector being divided.
	 * @param Divisor What to divide the int vector by.
	 * @return A new divided int vector.
	 */
	static TIntVector4 DivideAndRoundUp(TIntVector4 Lhs, IntType Divisor)
	{
		return TIntVector4(
			FMath::DivideAndRoundUp(Lhs.X, Divisor),
			FMath::DivideAndRoundUp(Lhs.Y, Divisor),
			FMath::DivideAndRoundUp(Lhs.Z, Divisor),
			FMath::DivideAndRoundUp(Lhs.W, Divisor));
	}

	static TIntVector4 DivideAndRoundUp(TIntVector4 Lhs, TIntVector4 Divisor)
	{
		return TIntVector4(
			FMath::DivideAndRoundUp(Lhs.X, Divisor.X),
			FMath::DivideAndRoundUp(Lhs.Y, Divisor.Y),
			FMath::DivideAndRoundUp(Lhs.Z, Divisor.Z),
			FMath::DivideAndRoundUp(Lhs.W, Divisor.W));
	}

	/**
	 * Gets the number of components a vector has.
	 *
	 * @return Number of components vector has.
	 */
	static int32 Num()
	{
		return 4;
	}

	/**
	 * Serializes the Vector4.
	 *
	 * @param Ar The archive to serialize into.
	 * @param Vector The vector to serialize.
	 * @return Reference to the Archive after serialization.
	 */
	friend FArchive& operator<<(FArchive& Ar, TIntVector4& Vector)
	{
		return Ar << Vector.X << Vector.Y << Vector.Z << Vector.W;
	}

	friend void operator<<(FStructuredArchive::FSlot Slot, TIntVector4& Vector)
	{
		FStructuredArchive::FRecord Record = Slot.EnterRecord();
		Record << SA_VALUE(TEXT("X"), Vector.X);
		Record << SA_VALUE(TEXT("Y"), Vector.Y);
		Record << SA_VALUE(TEXT("Z"), Vector.Z);
		Record << SA_VALUE(TEXT("W"), Vector.W);
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		if constexpr (std::is_same_v<IntType, int32>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, IntVector4, Int32Vector4, Int64Vector4);
		}
		else if constexpr (std::is_same_v<IntType, int64>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, IntVector4, Int64Vector4, Int32Vector4);
		}
		else if constexpr (std::is_same_v<IntType, uint32>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, UintVector4, Uint32Vector4, Uint64Vector4);
		}
		else if constexpr (std::is_same_v<IntType, uint64>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, UintVector4, Uint64Vector4, Uint32Vector4);
		}
		else
		{
			static_assert(sizeof(IntType) == 0, "Unimplemented");
			return false;
		}
	}
};

template <typename IntType>
const TIntVector4<IntType> TIntVector4<IntType>::ZeroValue(0, 0, 0, 0);

template <typename IntType>
const TIntVector4<IntType> TIntVector4<IntType>::NoneValue(INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE);

/**
 * Creates a hash value from an IntVector2.
 *
 * @param Vector the vector to create a hash value for
 * @return The hash value from the components
 */
template<typename T>
uint32 GetTypeHash(const TIntVector2<T>& Vector)
{
	// Note: this assumes there's no padding in Vector that could contain uncompared data.
	return FCrc::MemCrc32(&Vector, sizeof(Vector));
}

/**
 * Creates a hash value from an IntVector3.
 *
 * @param Vector the vector to create a hash value for
 * @return The hash value from the components
 */
template<typename T>
uint32 GetTypeHash(const TIntVector3<T>& Vector)
{
	// Note: this assumes there's no padding in Vector that could contain uncompared data.
	return FCrc::MemCrc_DEPRECATED(&Vector, sizeof(Vector));
}

/**
 * Creates a hash value from an IntVector4.
 *
 * @param Vector the vector to create a hash value for
 * @return The hash value from the components
 */
template<typename T>
uint32 GetTypeHash(const TIntVector4<T>& Vector)
{
	// Note: this assumes there's no padding in Vector that could contain uncompared data.
	return FCrc::MemCrc32(&Vector, sizeof(Vector));
}

} //! namespace UE::Math

template <> struct TIsPODType<FInt32Vector2>  { enum { Value = true }; };
template <> struct TIsPODType<FUint32Vector2> { enum { Value = true }; };
template <> struct TIsPODType<FInt32Vector3>  { enum { Value = true }; };
template <> struct TIsPODType<FUint32Vector3> { enum { Value = true }; };
template <> struct TIsPODType<FInt32Vector4>  { enum { Value = true }; };
template <> struct TIsPODType<FUint32Vector4> { enum { Value = true }; };

template <> struct TIsUECoreVariant<FInt32Vector2>  { enum { Value = true }; };
template <> struct TIsUECoreVariant<FUint32Vector2> { enum { Value = true }; };
template <> struct TIsUECoreVariant<FInt32Vector3>  { enum { Value = true }; };
template <> struct TIsUECoreVariant<FUint32Vector3> { enum { Value = true }; };
template <> struct TIsUECoreVariant<FInt32Vector4>  { enum { Value = true }; };
template <> struct TIsUECoreVariant<FUint32Vector4> { enum { Value = true }; };

template <> struct TIsPODType<FInt64Vector2> { enum { Value = true }; };
template <> struct TIsPODType<FUint64Vector2> { enum { Value = true }; };
template <> struct TIsPODType<FInt64Vector3> { enum { Value = true }; };
template <> struct TIsPODType<FUint64Vector3> { enum { Value = true }; };
template <> struct TIsPODType<FInt64Vector4> { enum { Value = true }; };
template <> struct TIsPODType<FUint64Vector4> { enum { Value = true }; };

template <> struct TIsUECoreVariant<FInt64Vector2> { enum { Value = true }; };
template <> struct TIsUECoreVariant<FUint64Vector2> { enum { Value = true }; };
template <> struct TIsUECoreVariant<FInt64Vector3> { enum { Value = true }; };
template <> struct TIsUECoreVariant<FUint64Vector3> { enum { Value = true }; };
template <> struct TIsUECoreVariant<FInt64Vector4> { enum { Value = true }; };
template <> struct TIsUECoreVariant<FUint64Vector4> { enum { Value = true }; };
