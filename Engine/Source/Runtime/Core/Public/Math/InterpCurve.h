// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Algo/MinElement.h"
#include "Containers/Array.h"
#include "Math/PolynomialRootSolver.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Math/Vector.h"
#include "Math/Quat.h"
#include "Math/TwoVectors.h"
#include "Math/InterpCurvePoint.h"

/**
 * Template for interpolation curves.
 *
 * @see FInterpCurvePoint
 * @todo Docs: FInterpCurve needs template and function documentation
 */
template<class T>
class FInterpCurve
{
public:

	/** Holds the collection of interpolation points. */
	TArray<FInterpCurvePoint<T>> Points;

	/** Specify whether the curve is looped or not */
	bool bIsLooped;

	/** Specify the offset from the last point's input key corresponding to the loop point */
	float LoopKeyOffset;

public:
	using ElementType = T;

	/** Default constructor. */
	[[nodiscard]] FInterpCurve() 
		: bIsLooped(false)
		, LoopKeyOffset(0.f)
	{ 
	}

public:

	/**
	 * Adds a new keypoint to the InterpCurve with the supplied In and Out value.
	 *
	 * @param InVal
	 * @param OutVal
	 * @return The index of the new key.
	 */
	int32 AddPoint( const float InVal, const T &OutVal );

	/**
	 * Moves a keypoint to a new In value.
	 *
	 * This may change the index of the keypoint, so the new key index is returned.
	 *
	 * @param PointIndex
	 * @param NewInVal
	 * @return
	 */
	int32 MovePoint( int32 PointIndex, float NewInVal );

	/** Clears all keypoints from InterpCurve. */
	void Reset();

	/** Set loop key for curve */
	void SetLoopKey( float InLoopKey );

	/** Clear loop key for curve */
	void ClearLoopKey();

	/** 
	 *	Evaluate the output for an arbitrary input value. 
	 *	For inputs outside the range of the keys, the first/last key value is assumed.
	 */
	[[nodiscard]] T Eval( const float InVal, const T& Default = T(ForceInit) ) const;

	/** 
	 *	Evaluate the derivative at a point on the curve.
	 */
	[[nodiscard]] T EvalDerivative( const float InVal, const T& Default = T(ForceInit) ) const;

	/** 
	 *	Evaluate the second derivative at a point on the curve.
	 */
	[[nodiscard]] T EvalSecondDerivative( const float InVal, const T& Default = T(ForceInit) ) const;

	/** 
	 * Find the nearest point on spline to the given point.
	 *
	 * @param PointInSpace - the given point
	 * @param OutDistanceSq - output - the squared distance between the given point and the closest found point.
	 * @return The key (the 't' parameter) of the nearest point. 
	 */
	float FindNearest( const T &PointInSpace, float& OutDistanceSq ) const;

	/**
	 * @deprecated Use FindNearest instead.
	 */
	float InaccurateFindNearest(const T& PointInSpace, float& OutDistanceSq) const;

	/**
	* Find the nearest point on spline to the given point.
	 *
	* @param PointInSpace - the given point
	* @param OutDistanceSq - output - the squared distance between the given point and the closest found point.
	* @param OutSegment - output - the nearest segment to the given point.
	* @return The key (the 't' parameter) of the nearest point.
	*/
	float FindNearest( const T &PointInSpace, float& OutDistanceSq, float& OutSegment ) const;

	/**
	 * @deprecated Use FindNearest instead.
	 */
	float InaccurateFindNearest(const T& PointInSpace, float& OutDistanceSq, float& OutSegment) const;

	/** 
	 * Find the nearest point (to the given point) on segment between Points[PtIdx] and Points[PtIdx+1]
	 *
	 * @param PointInSpace - the given point
	 * @return The key (the 't' parameter) of the found point. 
	 */
	float FindNearestOnSegment( const T &PointInSpace, int32 PtIdx, float& OutSquaredDistance ) const;

	/**
	 * @deprecated Use FindNearestOnSegment instead.
	 */
	float InaccurateFindNearestOnSegment(const T& PointInSpace, int32 PtIdx, float& OutSquaredDistance) const;

	/** Automatically set the tangents on the curve based on surrounding points */
	void AutoSetTangents(float Tension = 0.0f, bool bStationaryEndpoints = true);

	/** Calculate the min/max out value that can be returned by this InterpCurve. */
	void CalcBounds(T& OutMin, T& OutMax, const T& Default = T(ForceInit)) const;

	/** Returns the interval of key for the curve points */
	FFloatInterval GetKeyInterval() const;

public:

	/**
	 * Serializes the interp curve.
	 *
	 * @param Ar Reference to the serialization archive.
	 * @param Curve Reference to the interp curve being serialized.
	 *
	 * @return Reference to the Archive after serialization.
	 */
	friend FArchive& operator<<( FArchive& Ar, FInterpCurve& Curve )
	{
		// NOTE: This is not used often for FInterpCurves.  Most of the time these are serialized
		//   as inline struct properties in UnClass.cpp!

		Ar << Curve.Points;
		if (Ar.UEVer() >= VER_UE4_INTERPCURVE_SUPPORTS_LOOPING)
		{
			Ar << Curve.bIsLooped;
			Ar << Curve.LoopKeyOffset;
		}

		return Ar;
	}

	/**
	 * Compare equality of two FInterpCurves
	 */
	[[nodiscard]] friend bool operator==(const FInterpCurve& Curve1, const FInterpCurve& Curve2)
	{
		return (Curve1.Points == Curve2.Points &&
				Curve1.bIsLooped == Curve2.bIsLooped &&
				(!Curve1.bIsLooped || Curve1.LoopKeyOffset == Curve2.LoopKeyOffset));
	}

	/**
	 * Compare inequality of two FInterpCurves
	 */
	[[nodiscard]] friend bool operator!=(const FInterpCurve& Curve1, const FInterpCurve& Curve2)
	{
		return !(Curve1 == Curve2);
	}

	/**
	 * Finds the lower index of the two points whose input values bound the supplied input value.
	 */
	[[nodiscard]] int32 GetPointIndexForInputValue(const float InValue) const;
};


/* FInterpCurve inline functions
 *****************************************************************************/

template< class T > 
int32 FInterpCurve<T>::AddPoint( const float InVal, const T &OutVal )
{
	int32 i=0; for( i=0; i<Points.Num() && Points[i].InVal < InVal; i++);
	Points.InsertUninitialized(i);
	Points[i] = FInterpCurvePoint< T >(InVal, OutVal);
	return i;
}


template< class T > 
int32 FInterpCurve<T>::MovePoint( int32 PointIndex, float NewInVal )
{
	if( PointIndex < 0 || PointIndex >= Points.Num() )
		return PointIndex;

	const T OutVal = Points[PointIndex].OutVal;
	const EInterpCurveMode Mode = Points[PointIndex].InterpMode;
	const T ArriveTan = Points[PointIndex].ArriveTangent;
	const T LeaveTan = Points[PointIndex].LeaveTangent;

	Points.RemoveAt(PointIndex);

	const int32 NewPointIndex = AddPoint( NewInVal, OutVal );
	Points[NewPointIndex].InterpMode = Mode;
	Points[NewPointIndex].ArriveTangent = ArriveTan;
	Points[NewPointIndex].LeaveTangent = LeaveTan;

	return NewPointIndex;
}


template< class T > 
void FInterpCurve<T>::Reset()
{
	Points.Empty();
}


template <class T>
void FInterpCurve<T>::SetLoopKey(float InLoopKey)
{
	// Can't set a loop key if there are no points
	if (Points.Num() == 0)
	{
		bIsLooped = false;
		return;
	}

	const float LastInKey = Points.Last().InVal;
	if (InLoopKey > LastInKey)
	{
		// Calculate loop key offset from the input key of the final point
		bIsLooped = true;
		LoopKeyOffset = InLoopKey - LastInKey;
	}
	else
	{
		// Specified a loop key lower than the final point; turn off looping.
		bIsLooped = false;
	}
}


template <class T>
void FInterpCurve<T>::ClearLoopKey()
{
	bIsLooped = false;
}


template< class T >
int32 FInterpCurve<T>::GetPointIndexForInputValue(const float InValue) const
{
	const int32 NumPoints = Points.Num();
	const int32 LastPoint = NumPoints - 1;

	check(NumPoints > 0);

	if (InValue < Points[0].InVal)
	{
		return -1;
	}

	if (InValue >= Points[LastPoint].InVal)
	{
		return LastPoint;
	}

	int32 MinIndex = 0;
	int32 MaxIndex = NumPoints;

	while (MaxIndex - MinIndex > 1)
	{
		int32 MidIndex = (MinIndex + MaxIndex) / 2;

		if (Points[MidIndex].InVal <= InValue)
		{
			MinIndex = MidIndex;
		}
		else
		{
			MaxIndex = MidIndex;
		}
	}

	return MinIndex;
}


template< class T >
T FInterpCurve<T>::Eval(const float InVal, const T& Default) const
{
	const int32 NumPoints = Points.Num();
	const int32 LastPoint = NumPoints - 1;

	// If no point in curve, return the Default value we passed in.
	if (NumPoints == 0)
	{
		return Default;
	}

	// If we let NaNs in through here, they fail the check() on the Alpha between the two points.
	if (FPlatformMath::IsNaN(InVal))
	{
#if ENABLE_NAN_DIAGNOSTIC
		logOrEnsureNanError(TEXT("FInterpCurve<T>::Eval has InVal == NaN"));
#endif
		return Default;
	}

	// Binary search to find index of lower bound of input value
	const int32 Index = GetPointIndexForInputValue(InVal);

	// If before the first point, return its value
	if (Index == -1)
	{
		return Points[0].OutVal;
	}

	// If on or beyond the last point, return its value.
	if (Index == LastPoint)
	{
		if (!bIsLooped)
		{
			return Points[LastPoint].OutVal;
		}
		else if (InVal >= Points[LastPoint].InVal + LoopKeyOffset)
		{
			// Looped spline: last point is the same as the first point
			return Points[0].OutVal;
		}
	}

	// Somewhere within curve range - interpolate.
	check(Index >= 0 && ((bIsLooped && Index < NumPoints) || (!bIsLooped && Index < LastPoint)));
	const bool bLoopSegment = (bIsLooped && Index == LastPoint);
	const int32 NextIndex = bLoopSegment ? 0 : (Index + 1);

	const auto& PrevPoint = Points[Index];
	const auto& NextPoint = Points[NextIndex];

	const float Diff = bLoopSegment ? LoopKeyOffset : (NextPoint.InVal - PrevPoint.InVal);

	if (Diff > 0.0f && PrevPoint.InterpMode != CIM_Constant)
	{
		const float Alpha = (InVal - PrevPoint.InVal) / Diff;
		checkf(Alpha >= 0.0f && Alpha <= 1.0f, TEXT("Bad value in Eval(): in %f prev %f  diff %f alpha %f"), InVal, PrevPoint.InVal, Diff, Alpha);

		if (PrevPoint.InterpMode == CIM_Linear)
		{
			return FMath::Lerp(PrevPoint.OutVal, NextPoint.OutVal, Alpha);
		}
		else
		{
			return FMath::CubicInterp(PrevPoint.OutVal, PrevPoint.LeaveTangent * Diff, NextPoint.OutVal, NextPoint.ArriveTangent * Diff, Alpha);
		}
	}
	else
	{
		return Points[Index].OutVal;
	}
}


template< class T >
T FInterpCurve<T>::EvalDerivative(const float InVal, const T& Default) const
{
	const int32 NumPoints = Points.Num();
	const int32 LastPoint = NumPoints - 1;

	// If no point in curve, return the Default value we passed in.
	if (NumPoints == 0)
	{
		return Default;
	}

	// If we let NaNs in through here, they fail the check() on the Alpha between the two points.
	if (FPlatformMath::IsNaN(InVal))
	{
#if ENABLE_NAN_DIAGNOSTIC
		logOrEnsureNanError(TEXT("FInterpCurve<T>::EvalDerivative has InVal == NaN"));
#endif
		return Default;
	}
	
	// Binary search to find index of lower bound of input value
	const int32 Index = GetPointIndexForInputValue(InVal);

	// If before the first point, return its tangent value
	if (Index == -1)
	{
		return Points[0].LeaveTangent;
	}

	// If on or beyond the last point, return its tangent value.
	if (Index == LastPoint)
	{
		if (!bIsLooped)
		{
			return Points[LastPoint].ArriveTangent;
		}
		else if (InVal >= Points[LastPoint].InVal + LoopKeyOffset)
		{
			// Looped spline: last point is the same as the first point
			return Points[0].ArriveTangent;
		}
	}

	// Somewhere within curve range - interpolate.
	check(Index >= 0 && ((bIsLooped && Index < NumPoints) || (!bIsLooped && Index < LastPoint)));
	const bool bLoopSegment = (bIsLooped && Index == LastPoint);
	const int32 NextIndex = bLoopSegment ? 0 : (Index + 1);

	const auto& PrevPoint = Points[Index];
	const auto& NextPoint = Points[NextIndex];

	const float Diff = bLoopSegment ? LoopKeyOffset : (NextPoint.InVal - PrevPoint.InVal);

	if (Diff > 0.0f && PrevPoint.InterpMode != CIM_Constant)
	{
		if (PrevPoint.InterpMode == CIM_Linear)
		{
			return (NextPoint.OutVal - PrevPoint.OutVal) / Diff;
		}
		else
		{
			const float Alpha = (InVal - PrevPoint.InVal) / Diff;
			checkf(Alpha >= 0.0f && Alpha <= 1.0f, TEXT("Bad value in EvalDerivative(): in %f prev %f  diff %f alpha %f"), InVal, PrevPoint.InVal, Diff, Alpha);

			return FMath::CubicInterpDerivative(PrevPoint.OutVal, PrevPoint.LeaveTangent * Diff, NextPoint.OutVal, NextPoint.ArriveTangent * Diff, Alpha) / Diff;
		}
	}
	else
	{
		// Derivative of a constant is zero
		return T(ForceInit);
	}
}

template< class T >
T FInterpCurve<T>::EvalSecondDerivative(const float InVal, const T& Default) const
{
	const int32 NumPoints = Points.Num();
	const int32 LastPoint = NumPoints - 1;

	// If no point in curve, return the Default value we passed in.
	if (NumPoints == 0)
	{
		return Default;
	}

	// If we let NaNs in through here, they fail the check() on the Alpha between the two points.
	if (FPlatformMath::IsNaN(InVal))
	{
#if ENABLE_NAN_DIAGNOSTIC
		logOrEnsureNanError(TEXT("FInterpCurve<T>::EvalSecondDerivative has InVal == NaN"));
#endif
		return Default;
	}
	
	// Binary search to find index of lower bound of input value
	const int32 Index = GetPointIndexForInputValue(InVal);

	// If before the first point, return 0
	if (Index == -1)
	{
		return T(ForceInit);
	}

	// If on or beyond the last point, return 0
	if (Index == LastPoint)
	{
		if (!bIsLooped || (InVal >= Points[LastPoint].InVal + LoopKeyOffset))
		{
			return T(ForceInit);
		}
	}

	// Somewhere within curve range - interpolate.
	check(Index >= 0 && ((bIsLooped && Index < NumPoints) || (!bIsLooped && Index < LastPoint)));
	const bool bLoopSegment = (bIsLooped && Index == LastPoint);
	const int32 NextIndex = bLoopSegment ? 0 : (Index + 1);

	const auto& PrevPoint = Points[Index];
	const auto& NextPoint = Points[NextIndex];

	const float Diff = bLoopSegment ? LoopKeyOffset : (NextPoint.InVal - PrevPoint.InVal);

	if (Diff > 0.0f && PrevPoint.InterpMode != CIM_Constant)
	{
		if (PrevPoint.InterpMode == CIM_Linear)
		{
			// No change in tangent, return 0.
			return T(ForceInit);
		}
		else
		{
			const float Alpha = (InVal - PrevPoint.InVal) / Diff;
			checkf(Alpha >= 0.0f && Alpha <= 1.0f, TEXT("Bad value in EvalSecondDerivative(): in %f prev %f  diff %f alpha %f"), InVal, PrevPoint.InVal, Diff, Alpha);

			return FMath::CubicInterpSecondDerivative(PrevPoint.OutVal, PrevPoint.LeaveTangent * Diff, NextPoint.OutVal, NextPoint.ArriveTangent * Diff, Alpha) / (Diff * Diff);
		}
	}
	else
	{
		// Second derivative of a constant is zero
		return T(ForceInit);
	}
}

template< class T >
float FInterpCurve<T>::FindNearest(const T& PointInSpace, float& OutDistanceSq) const
{
	float OutSegment;
	return FindNearest(PointInSpace, OutDistanceSq, OutSegment);
}

template< class T >
float FInterpCurve<T>::InaccurateFindNearest(const T &PointInSpace, float& OutDistanceSq) const
{
	float OutSegment;
	return FindNearest(PointInSpace, OutDistanceSq, OutSegment);
}

template< class T >
float FInterpCurve<T>::FindNearest(const T &PointInSpace, float& OutDistanceSq, float& OutSegment) const		// LWC_TODO: Precision loss
{
	const int32 NumPoints = Points.Num();
	const int32 NumSegments = bIsLooped ? NumPoints : NumPoints - 1;

	if (NumPoints > 1)
	{
		float BestDistanceSq;
		float BestResult = FindNearestOnSegment(PointInSpace, 0, BestDistanceSq);
		float BestSegment = 0;
		for (int32 Segment = 1; Segment < NumSegments; ++Segment)
		{
			float LocalDistanceSq;
			float LocalResult = FindNearestOnSegment(PointInSpace, Segment, LocalDistanceSq);
			if (LocalDistanceSq < BestDistanceSq)
			{
				BestDistanceSq = LocalDistanceSq;
				BestResult = LocalResult;
				BestSegment = (float)Segment;
			}
		}
		OutDistanceSq = BestDistanceSq;
		OutSegment = BestSegment;
		return BestResult;
	}

	if (NumPoints == 1)
	{
		OutDistanceSq = static_cast<float>((PointInSpace - Points[0].OutVal).SizeSquared());
		OutSegment = 0;
		return Points[0].InVal;
	}

	return 0.0f;
}

template< class T >
float FInterpCurve<T>::InaccurateFindNearest(const T& PointInSpace, float& OutDistanceSq, float& OutSegment) const
{
	return FindNearest(PointInSpace, OutDistanceSq, OutSegment);
}

template< class T >
float FInterpCurve<T>::FindNearestOnSegment(const T& PointInSpace, int32 PtIdx, float& OutSquaredDistance) const
{
	const int32 NumPoints = Points.Num();
	const int32 LastPoint = NumPoints - 1;
	const int32 NextPtIdx = (bIsLooped && PtIdx == LastPoint) ? 0 : (PtIdx + 1);
	check(PtIdx >= 0 && ((bIsLooped && PtIdx < NumPoints) || (!bIsLooped && PtIdx < LastPoint)));

	const float NextInVal = (bIsLooped && PtIdx == LastPoint) ? (Points[LastPoint].InVal + LoopKeyOffset) : Points[NextPtIdx].InVal;

	if (CIM_Constant == Points[PtIdx].InterpMode)
	{
		const float Distance1 = static_cast<float>((Points[PtIdx].OutVal - PointInSpace).SizeSquared());
		const float Distance2 = static_cast<float>((Points[NextPtIdx].OutVal - PointInSpace).SizeSquared());
		if (Distance1 < Distance2)
		{
			OutSquaredDistance = Distance1;
			return Points[PtIdx].InVal;
		}
		OutSquaredDistance = Distance2;
		return NextInVal;
	}

	const float Diff = NextInVal - Points[PtIdx].InVal;
	if (CIM_Linear == Points[PtIdx].InterpMode)
	{
		// like in function: FMath::ClosestPointOnLine
		const float A = static_cast<float>((Points[PtIdx].OutVal - PointInSpace) | (Points[NextPtIdx].OutVal - Points[PtIdx].OutVal));
		const float B = static_cast<float>((Points[NextPtIdx].OutVal - Points[PtIdx].OutVal).SizeSquared());
		const float V = FMath::Clamp(-A / B, 0.f, 1.f);
		OutSquaredDistance = static_cast<float>((FMath::Lerp(Points[PtIdx].OutVal, Points[NextPtIdx].OutVal, V) - PointInSpace).SizeSquared());
		return V * Diff + Points[PtIdx].InVal;
	}

	{
		const TArray<FInterpCurvePoint<T>>& PointsT = Points;

		// Get the cubic's control points, shifted so PointInSpace is at the origin
		const T P0 = PointsT[PtIdx].OutVal - PointInSpace;
		const T P1 = PointsT[NextPtIdx].OutVal - PointInSpace;
		const T T0 = PointsT[PtIdx].LeaveTangent * Diff;
		const T T1 = PointsT[NextPtIdx].ArriveTangent * Diff;
		const T CubicCoeffs[4] = { P0, T0, -3 * P0 - 2 * T0 - T1 + 3 * P1, 2 * P0 + T0 + T1 - 2 * P1 };
		
		// Curve is closest to PointInSpace when (Curve - PointInSpace).Dot(CurveDerivative) == 0
		// Since we pre-subtracted PointInSpace, this becomes Curve.Dot(CurveDerivative) == 0,
		// which expands out to the degree-5 polynomial w/ the below coefficients
		double Coeffs[6];
		Coeffs[5] = 3 * CubicCoeffs[3].Dot(CubicCoeffs[3]);
		Coeffs[4] = 5 * CubicCoeffs[3].Dot(CubicCoeffs[2]);
		Coeffs[3] = 4 * CubicCoeffs[3].Dot(CubicCoeffs[1]) + 2 * CubicCoeffs[2].Dot(CubicCoeffs[2]);
		Coeffs[2] = 3 * CubicCoeffs[2].Dot(CubicCoeffs[1]) + 3 * CubicCoeffs[3].Dot(CubicCoeffs[0]);
		Coeffs[1] = CubicCoeffs[1].Dot(CubicCoeffs[1]) + 2 * CubicCoeffs[0].Dot(CubicCoeffs[2]);
		Coeffs[0] = CubicCoeffs[0].Dot(CubicCoeffs[1]);

		// Test the endpoints first -- recall P0 and P1 already have PointInSpace subtracted out
		float BestDistSq = static_cast<float>(P0.SizeSquared());
		float BestParam = 0;
		float EndDistSq = static_cast<float>(P1.SizeSquared());
		if (EndDistSq < BestDistSq)
		{
			BestParam = 1;
			BestDistSq = EndDistSq;
		}
		// Check the roots
		UE::Math::TPolynomialRootSolver<double, 5> RootSolver(Coeffs, 0, 1);
		for (int32 RootIdx = 0; RootIdx < RootSolver.Roots.Num(); ++RootIdx)
		{
			T FoundPoint = FMath::CubicInterp(P0, T0, P1, T1, RootSolver.Roots[RootIdx]);
			float RootDistSq = static_cast<float>(FoundPoint.SizeSquared());
			if (RootDistSq < BestDistSq)
			{
				BestParam = static_cast<float>(RootSolver.Roots[RootIdx]);
				BestDistSq = RootDistSq;
			}
		}
		// Return the best parameter value for the segment, shifted back to overall curve parameter space
		OutSquaredDistance = BestDistSq;
		return BestParam * Diff + Points[PtIdx].InVal;

	}
}

template< class T >
float FInterpCurve<T>::InaccurateFindNearestOnSegment(const T& PointInSpace, int32 PtIdx, float& OutSquaredDistance) const
{
	return FindNearestOnSegment(PointInSpace, PtIdx, OutSquaredDistance);
}

template< class T >
void FInterpCurve<T>::AutoSetTangents(float Tension, bool bStationaryEndpoints)
{
	const int32 NumPoints = Points.Num();
	const int32 LastPoint = NumPoints - 1;

	// Iterate over all points in this InterpCurve
	for (int32 PointIndex = 0; PointIndex < NumPoints; PointIndex++)
	{
		const int32 PrevIndex = (PointIndex == 0) ? (bIsLooped ? LastPoint : 0) : (PointIndex - 1);
		const int32 NextIndex = (PointIndex == LastPoint) ? (bIsLooped ? 0 : LastPoint) : (PointIndex + 1);

		auto& ThisPoint = Points[PointIndex];
		const auto& PrevPoint = Points[PrevIndex];
		const auto& NextPoint = Points[NextIndex];

		if (ThisPoint.InterpMode == CIM_CurveAuto || ThisPoint.InterpMode == CIM_CurveAutoClamped)
		{
			if (bStationaryEndpoints && (PointIndex == 0 || (PointIndex == LastPoint && !bIsLooped)))
			{
				// Start and end points get zero tangents if bStationaryEndpoints is true
				ThisPoint.ArriveTangent = T(ForceInit);
				ThisPoint.LeaveTangent = T(ForceInit);
			}
			else if (PrevPoint.IsCurveKey())
			{
				const bool bWantClamping = (ThisPoint.InterpMode == CIM_CurveAutoClamped);
				T Tangent;

				const float PrevTime = (bIsLooped && PointIndex == 0) ? (ThisPoint.InVal - LoopKeyOffset) : PrevPoint.InVal;
				const float NextTime = (bIsLooped && PointIndex == LastPoint) ? (ThisPoint.InVal + LoopKeyOffset) : NextPoint.InVal;

				ComputeCurveTangent(
					PrevTime,			// Previous time
					PrevPoint.OutVal,	// Previous point
					ThisPoint.InVal,	// Current time
					ThisPoint.OutVal,	// Current point
					NextTime,			// Next time
					NextPoint.OutVal,	// Next point
					Tension,			// Tension
					bWantClamping,		// Want clamping?
					Tangent);			// Out

				ThisPoint.ArriveTangent = Tangent;
				ThisPoint.LeaveTangent = Tangent;
			}
			else
			{
				// Following on from a line or constant; set curve tangent equal to that so there are no discontinuities
				ThisPoint.ArriveTangent = PrevPoint.ArriveTangent;
				ThisPoint.LeaveTangent = PrevPoint.LeaveTangent;
			}
		}
		else if (ThisPoint.InterpMode == CIM_Linear)
		{
			ThisPoint.LeaveTangent = NextPoint.OutVal - ThisPoint.OutVal;

			// Following from a curve, we should set the tangents equal so that there are no discontinuities
			ThisPoint.ArriveTangent = PrevPoint.IsCurveKey() ? ThisPoint.LeaveTangent : ThisPoint.OutVal - PrevPoint.OutVal;
		}
		else if (ThisPoint.InterpMode == CIM_Constant)
		{
			ThisPoint.ArriveTangent = T(ForceInit);
			ThisPoint.LeaveTangent = T(ForceInit);
		}
	}
}


template< class T >
void FInterpCurve<T>::CalcBounds(T& OutMin, T& OutMax, const T& Default) const
{
	const int32 NumPoints = Points.Num();

	if (NumPoints == 0)
	{
		OutMin = Default;
		OutMax = Default;
	}
	else if (NumPoints == 1)
	{
		OutMin = Points[0].OutVal;
		OutMax = Points[0].OutVal;
	}
	else
	{
		OutMin = Points[0].OutVal;
		OutMax = Points[0].OutVal;

		const int32 NumSegments = bIsLooped ? NumPoints : (NumPoints - 1);

		for (int32 Index = 0; Index < NumSegments; Index++)
		{
			const int32 NextIndex = (Index == NumPoints - 1) ? 0 : (Index + 1);
			CurveFindIntervalBounds(Points[Index], Points[NextIndex], OutMin, OutMax, 0.0f);
		}
	}
}

template <class T>
FFloatInterval FInterpCurve<T>::GetKeyInterval() const
{
	if (Points.IsEmpty())
	{
		return {};
	}

	return FFloatInterval(Points[0].InVal, Points.Last().InVal);
}

/* Common type definitions
 *****************************************************************************/

#define DEFINE_INTERPCURVE_WRAPPER_STRUCT(Name, ElementType) \
	struct Name : FInterpCurve<ElementType> \
	{ \
	private: \
		typedef FInterpCurve<ElementType> Super; \
	 \
	public: \
		Name() \
			: Super() \
		{ \
		} \
		 \
		Name(const Super& Other) \
			: Super( Other ) \
		{ \
		} \
	}; \
	 \
	template <> \
	struct TIsBitwiseConstructible<Name, FInterpCurve<ElementType>> \
	{ \
		enum { Value = true }; \
	}; \
	 \
	template <> \
	struct TIsBitwiseConstructible<FInterpCurve<ElementType>, Name> \
	{ \
		enum { Value = true }; \
	};

DEFINE_INTERPCURVE_WRAPPER_STRUCT(FInterpCurveFloat,       float)
DEFINE_INTERPCURVE_WRAPPER_STRUCT(FInterpCurveVector2D,    FVector2D)
DEFINE_INTERPCURVE_WRAPPER_STRUCT(FInterpCurveVector,      FVector)
DEFINE_INTERPCURVE_WRAPPER_STRUCT(FInterpCurveQuat,        FQuat)
DEFINE_INTERPCURVE_WRAPPER_STRUCT(FInterpCurveTwoVectors,  FTwoVectors)
DEFINE_INTERPCURVE_WRAPPER_STRUCT(FInterpCurveLinearColor, FLinearColor)
