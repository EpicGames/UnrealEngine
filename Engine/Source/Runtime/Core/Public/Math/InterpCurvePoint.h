// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Containers/EnumAsByte.h"
#include "Math/Vector.h"
#include "Math/Quat.h"
#include "Math/TwoVectors.h"
#include "UObject/CoreReflectedTypeAccessors.h"

enum EInterpCurveMode : int
{
	/** A straight line between two keypoint values. */
	CIM_Linear,

	/** A cubic-hermite curve between two keypoints, using Arrive/Leave tangents. These tangents will be automatically
	    updated when points are moved, etc.  Tangents are unclamped and will plateau at curve start and end points. */
	CIM_CurveAuto,

	/** The out value is held constant until the next key, then will jump to that value. */
	CIM_Constant,

	/** A smooth curve just like CIM_Curve, but tangents are not automatically updated so you can have manual control over them (eg. in Curve Editor). */
	CIM_CurveUser,

	/** A curve like CIM_Curve, but the arrive and leave tangents are not forced to be the same, so you can create a 'corner' at this key. */
	CIM_CurveBreak,

	/** A cubic-hermite curve between two keypoints, using Arrive/Leave tangents. These tangents will be automatically
	    updated when points are moved, etc.  Tangents are clamped and will plateau at curve start and end points. */
	CIM_CurveAutoClamped,

	/** Invalid or unknown curve type. */
	CIM_Unknown
};

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<EInterpCurveMode>();
#endif


/**
 * Template for interpolation points.
 *
 * Interpolation points are used for describing the shape of interpolation curves.
 *
 * @see FInterpCurve
 * @todo Docs: FInterpCurvePoint needs better function documentation.
 */
template< class T > class FInterpCurvePoint
{
public:

	/** Float input value that corresponds to this key (eg. time). */
	float InVal;

	/** Output value of templated type when input is equal to InVal. */
	T OutVal;

	/** Tangent of curve arrive this point. */
	T ArriveTangent; 

	/** Tangent of curve leaving this point. */
	T LeaveTangent; 

	/** Interpolation mode between this point and the next one. @see EInterpCurveMode */
	TEnumAsByte<EInterpCurveMode> InterpMode; 

public:

	/**
	 * Default constructor (no initialization).
	 */
	[[nodiscard]] FInterpCurvePoint() = default;

	/** 
	 * Constructor 
	 *
	 * @param In input value that corresponds to this key
	 * @param Out Output value of templated type
	 * @note uses linear interpolation
	 */
	[[nodiscard]] FInterpCurvePoint( const float In, const T &Out );

	/** 
	 * Constructor 
	 *
	 * @param In input value that corresponds to this key
	 * @param Out Output value of templated type
	 * @param InArriveTangent Tangent of curve arriving at this point. 
	 * @param InLeaveTangent Tangent of curve leaving from this point.
	 * @param InInterpMode interpolation mode to use
	 */
	[[nodiscard]] FInterpCurvePoint( const float In, const T &Out, const T &InArriveTangent, const T &InLeaveTangent, const EInterpCurveMode InInterpMode );

	/**
	 * Constructor which initializes all components to zero.
	 *
	 * @param EForceInit Force init enum
	 */
	[[nodiscard]] explicit UE_FORCEINLINE_HINT FInterpCurvePoint(EForceInit);

public:

	/** @return true if the key value is using a curve interp mode, otherwise false */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsCurveKey() const;

public:

	/**
	 * Serializes the Curve Point.
	 *
	 * @param Ar Reference to the serialization archive.
	 * @param Point Reference to the curve point being serialized.
	 *
	 * @return Reference to the Archive after serialization.
	 */
	friend FArchive& operator<<( FArchive& Ar, FInterpCurvePoint& Point )
	{
		Ar << Point.InVal << Point.OutVal;
		Ar << Point.ArriveTangent << Point.LeaveTangent;
		Ar << Point.InterpMode;
		return Ar;
	}

	/**
	 * Compare equality of two Curve Points
	 */
	[[nodiscard]] friend bool operator==( const FInterpCurvePoint& Point1, const FInterpCurvePoint& Point2 )
	{
		return (Point1.InVal == Point2.InVal &&
				Point1.OutVal == Point2.OutVal &&
				Point1.ArriveTangent == Point2.ArriveTangent &&
				Point1.LeaveTangent == Point2.LeaveTangent &&
				Point1.InterpMode == Point2.InterpMode);
	}

	/**
	 * Compare inequality of two Curve Points
	 */
	[[nodiscard]] friend bool operator!=(const FInterpCurvePoint& Point1, const FInterpCurvePoint& Point2)
	{
		return !(Point1 == Point2);
	}
};


/* FInterpCurvePoint inline functions
 *****************************************************************************/

template< class T > 
inline FInterpCurvePoint<T>::FInterpCurvePoint( const float In, const T &Out )
	: InVal(In)
	, OutVal(Out)
{
	FMemory::Memset( &ArriveTangent, 0, sizeof(T) );	
	FMemory::Memset( &LeaveTangent, 0, sizeof(T) );

	InterpMode = CIM_Linear;
}


template< class T > 
inline FInterpCurvePoint<T>::FInterpCurvePoint( const float In, const T &Out, const T &InArriveTangent, const T &InLeaveTangent, const EInterpCurveMode InInterpMode)
	: InVal(In)
	, OutVal(Out)
	, ArriveTangent(InArriveTangent)
	, LeaveTangent(InLeaveTangent)
	, InterpMode(InInterpMode)
{ }

template< class T >
inline FInterpCurvePoint<T>::FInterpCurvePoint(EForceInit)
{
	InVal = 0.0f;
	FMemory::Memset(&OutVal, 0, sizeof(T));
	FMemory::Memset(&ArriveTangent, 0, sizeof(T));
	FMemory::Memset(&LeaveTangent, 0, sizeof(T));
	InterpMode = CIM_Linear;
}

template< class T > 
UE_FORCEINLINE_HINT bool FInterpCurvePoint<T>::IsCurveKey() const
{
	return ((InterpMode == CIM_CurveAuto) || (InterpMode == CIM_CurveAutoClamped) || (InterpMode == CIM_CurveUser) || (InterpMode == CIM_CurveBreak));
}

/**
 * Clamps a tangent formed by the specified control point values
 */
CORE_API float ClampFloatTangent( float PrevPointVal, float PrevTime, float CurPointVal, float CurTime, float NextPointVal, float NextTime );


/** Computes Tangent for a curve segment */
template< class T, class U > 
inline void AutoCalcTangent( const T& PrevP, const T& P, const T& NextP, const U& Tension, T& OutTan )
{
	OutTan = (1.f - Tension) * ( (P - PrevP) + (NextP - P) );
}


/**
 * This actually returns the control point not a tangent. This is expected by the CubicInterp function for Quaternions
 */
template< class U > 
inline void AutoCalcTangent( const FQuat& PrevP, const FQuat& P, const FQuat& NextP, const U& Tension, FQuat& OutTan  )
{
	FQuat::CalcTangents(PrevP, P, NextP, Tension, OutTan);
}


/** Computes a tangent for the specified control point.  General case, doesn't support clamping. */
template< class T >
inline void ComputeCurveTangent( float PrevTime, const T& PrevPoint,
							float CurTime, const T& CurPoint,
							float NextTime, const T& NextPoint,
							float Tension,
							bool bWantClamping,
							T& OutTangent )
{
	// NOTE: Clamping not supported for non-float vector types (bWantClamping is ignored)

	AutoCalcTangent( PrevPoint, CurPoint, NextPoint, Tension, OutTangent );

	const float PrevToNextTimeDiff = FMath::Max< float >( UE_KINDA_SMALL_NUMBER, NextTime - PrevTime );

	OutTangent /= PrevToNextTimeDiff;
}


/**
 * Computes a tangent for the specified control point; supports clamping, but only works
 * with floats or contiguous arrays of floats.
 */
template< class T >
inline void ComputeClampableFloatVectorCurveTangent( float PrevTime, const T& PrevPoint,
												float CurTime, const T& CurPoint,
												float NextTime, const T& NextPoint,
												float Tension,
												bool bWantClamping,
												T& OutTangent )
{
	// Clamp the tangents if we need to do that
	if( bWantClamping )
	{
		// NOTE: We always treat the type as an array of floats
		float* PrevPointVal = ( float* )&PrevPoint;
		float* CurPointVal = ( float* )&CurPoint;
		float* NextPointVal = ( float* )&NextPoint;
		float* OutTangentVal = ( float* )&OutTangent;
		for( int32 CurValPos = 0; CurValPos < sizeof( T ); CurValPos += sizeof( float ) )
		{
			// Clamp it!
			const float ClampedTangent =
				ClampFloatTangent(
					*PrevPointVal, PrevTime,
					*CurPointVal, CurTime,
					*NextPointVal, NextTime );

			// Apply tension value
			*OutTangentVal = ( 1.0f - Tension ) * ClampedTangent;


			// Advance pointers
			++OutTangentVal;
			++PrevPointVal;
			++CurPointVal;
			++NextPointVal;
		}
	}
	else
	{
		// No clamping needed
		AutoCalcTangent( PrevPoint, CurPoint, NextPoint, Tension, OutTangent );

		const float PrevToNextTimeDiff = FMath::Max< float >( UE_KINDA_SMALL_NUMBER, NextTime - PrevTime );

		OutTangent /= PrevToNextTimeDiff;
	}
}


/** Computes a tangent for the specified control point.  Special case for float types; supports clamping. */
inline void ComputeCurveTangent( float PrevTime, const float& PrevPoint,
									float CurTime, const float& CurPoint,
									float NextTime, const float& NextPoint,
									float Tension,
									bool bWantClamping,
									float& OutTangent )
{
	ComputeClampableFloatVectorCurveTangent(
		PrevTime, PrevPoint,
		CurTime, CurPoint,
		NextTime, NextPoint,
		Tension, bWantClamping, OutTangent );
}


/** Computes a tangent for the specified control point.  Special case for FVector types; supports clamping. */
inline void ComputeCurveTangent( float PrevTime, const FVector& PrevPoint,
									float CurTime, const FVector& CurPoint,
									float NextTime, const FVector& NextPoint,
									float Tension,
									bool bWantClamping,
									FVector& OutTangent )
{
	ComputeClampableFloatVectorCurveTangent(
		PrevTime, PrevPoint,
		CurTime, CurPoint,
		NextTime, NextPoint,
		Tension, bWantClamping, OutTangent );
}


/** Computes a tangent for the specified control point.  Special case for FVector2D types; supports clamping. */
inline void ComputeCurveTangent( float PrevTime, const FVector2D& PrevPoint,
									float CurTime, const FVector2D& CurPoint,
									float NextTime, const FVector2D& NextPoint,
									float Tension,
									bool bWantClamping,
									FVector2D& OutTangent )
{
	ComputeClampableFloatVectorCurveTangent(
		PrevTime, PrevPoint,
		CurTime, CurPoint,
		NextTime, NextPoint,
		Tension, bWantClamping, OutTangent );
}


/** Computes a tangent for the specified control point.  Special case for FTwoVectors types; supports clamping. */
inline void ComputeCurveTangent( float PrevTime, const FTwoVectors& PrevPoint,
									float CurTime, const FTwoVectors& CurPoint,
									float NextTime, const FTwoVectors& NextPoint,
									float Tension,
									bool bWantClamping,
									FTwoVectors& OutTangent )
{
	ComputeClampableFloatVectorCurveTangent(
		PrevTime, PrevPoint,
		CurTime, CurPoint,
		NextTime, NextPoint,
		Tension, bWantClamping, OutTangent );
}

/**
 * Calculate bounds of float intervals
 *
 * @param Start interp curve point at Start
 * @param End interp curve point at End
 * @param CurrentMin Input and Output could be updated if needs new interval minimum bound
 * @param CurrentMax  Input and Output could be updated if needs new interval maximmum bound
 */
void CORE_API CurveFloatFindIntervalBounds( const FInterpCurvePoint<float>& Start, const FInterpCurvePoint<float>& End, float& CurrentMin, float& CurrentMax );


/**
 * Calculate bounds of 2D vector intervals
 *
 * @param Start interp curve point at Start
 * @param End interp curve point at End
 * @param CurrentMin Input and Output could be updated if needs new interval minimum bound
 * @param CurrentMax  Input and Output could be updated if needs new interval maximmum bound
 */
void CORE_API CurveVector2DFindIntervalBounds( const FInterpCurvePoint<FVector2D>& Start, const FInterpCurvePoint<FVector2D>& End, FVector2D& CurrentMin, FVector2D& CurrentMax );


/**
 * Calculate bounds of vector intervals
 *
 * @param Start interp curve point at Start
 * @param End interp curve point at End
 * @param CurrentMin Input and Output could be updated if needs new interval minimum bound
 * @param CurrentMax  Input and Output could be updated if needs new interval maximmum bound
 */
void CORE_API CurveVectorFindIntervalBounds( const FInterpCurvePoint<FVector>& Start, const FInterpCurvePoint<FVector>& End, FVector& CurrentMin, FVector& CurrentMax );


/**
 * Calculate bounds of twovector intervals
 *
 * @param Start interp curve point at Start
 * @param End interp curve point at End
 * @param CurrentMin Input and Output could be updated if needs new interval minimum bound
 * @param CurrentMax  Input and Output could be updated if needs new interval maximmum bound
 */
void CORE_API CurveTwoVectorsFindIntervalBounds(const FInterpCurvePoint<FTwoVectors>& Start, const FInterpCurvePoint<FTwoVectors>& End, FTwoVectors& CurrentMin, FTwoVectors& CurrentMax);


/**
 * Calculate bounds of color intervals
 *
 * @param Start interp curve point at Start
 * @param End interp curve point at End
 * @param CurrentMin Input and Output could be updated if needs new interval minimum bound
 * @param CurrentMax  Input and Output could be updated if needs new interval maximmum bound
 */
void CORE_API CurveLinearColorFindIntervalBounds( const FInterpCurvePoint<FLinearColor>& Start, const FInterpCurvePoint<FLinearColor>& End, FLinearColor& CurrentMin, FLinearColor& CurrentMax );


template< class T, class U > 
inline void CurveFindIntervalBounds( const FInterpCurvePoint<T>& Start, const FInterpCurvePoint<T>& End, T& CurrentMin, T& CurrentMax, const U& Dummy )
{ }


template< class U > 
inline void CurveFindIntervalBounds( const FInterpCurvePoint<float>& Start, const FInterpCurvePoint<float>& End, float& CurrentMin, float& CurrentMax, const U& Dummy )
{
	CurveFloatFindIntervalBounds(Start, End, CurrentMin, CurrentMax);
}


template< class U > 
void CurveFindIntervalBounds( const FInterpCurvePoint<FVector2D>& Start, const FInterpCurvePoint<FVector2D>& End, FVector2D& CurrentMin, FVector2D& CurrentMax, const U& Dummy )
{
	CurveVector2DFindIntervalBounds(Start, End, CurrentMin, CurrentMax);
}


template< class U > 
inline void CurveFindIntervalBounds( const FInterpCurvePoint<FVector>& Start, const FInterpCurvePoint<FVector>& End, FVector& CurrentMin, FVector& CurrentMax, const U& Dummy )
{
	CurveVectorFindIntervalBounds(Start, End, CurrentMin, CurrentMax);
}


template< class U > 
inline void CurveFindIntervalBounds( const FInterpCurvePoint<FTwoVectors>& Start, const FInterpCurvePoint<FTwoVectors>& End, FTwoVectors& CurrentMin, FTwoVectors& CurrentMax, const U& Dummy )
{
	CurveTwoVectorsFindIntervalBounds(Start, End, CurrentMin, CurrentMax);
}


template< class U > 
inline void CurveFindIntervalBounds( const FInterpCurvePoint<FLinearColor>& Start, const FInterpCurvePoint<FLinearColor>& End, FLinearColor& CurrentMin, FLinearColor& CurrentMax, const U& Dummy )
{
	CurveLinearColorFindIntervalBounds(Start, End, CurrentMin, CurrentMax);
}

// Native implementation of NOEXPORT FInterpCurvePoint structures
typedef FInterpCurvePoint<float> FInterpCurvePointFloat;
typedef FInterpCurvePoint<FVector2D> FInterpCurvePointVector2D;
typedef FInterpCurvePoint<FVector> FInterpCurvePointVector;
typedef FInterpCurvePoint<FQuat> FInterpCurvePointQuat;
typedef FInterpCurvePoint<FTwoVectors> FInterpCurvePointTwoVectors;
typedef FInterpCurvePoint<FLinearColor> FInterpCurvePointLinearColor;
