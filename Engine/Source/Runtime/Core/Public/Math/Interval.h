// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/IsArithmetic.h"
#include "Templates/UnrealTypeTraits.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"

/**
 * Type traits for Arithmetic interval.
 */
template <typename ElementType> struct TIntervalTraits
{
	static_assert(::TIsArithmetic<ElementType>::Value, "Incompatible TInterval element type.");

	static ElementType Max()
	{
		return TNumericLimits<ElementType>::Max();
	}

	static ElementType Lowest()
	{
		return TNumericLimits<ElementType>::Lowest();
	}
};

/**
 * Template for numeric interval
 */
template<typename ElementType> struct TInterval
{
	/** Holds the lower bound of the interval. */
	ElementType Min;
	
	/** Holds the upper bound of the interval. */
	ElementType Max;
		
public:

	/**
	 * Default constructor.
	 *
	 * The interval is invalid
	 */
	[[nodiscard]] TInterval()
		: Min(TIntervalTraits<ElementType>::Max())
		, Max(TIntervalTraits<ElementType>::Lowest())
	{ }

    /**
	 * Creates and initializes a new interval with the specified lower and upper bounds.
	 *
	 * @param InMin The lower bound of the constructed interval.
	 * @param InMax The upper bound of the constructed interval.
	 */
	[[nodiscard]] TInterval( ElementType InMin, ElementType InMax )
		: Min(InMin)
		, Max(InMax)
	{ }

public:

	/**
	 * Offset the interval by adding X.
	 *
	 * @param X The offset.
	 */
	void operator+= ( ElementType X )
	{
		if (IsValid())
		{
			Min += X;
			Max += X;
		}
	}

	/**
	 * Offset the interval by subtracting X.
	 *
	 * @param X The offset.
	 */
	void operator-= ( ElementType X )
	{
		if (IsValid())
		{
			Min -= X;
			Max -= X;
		}
	}

	/**
	 * Compare two intervals for equality.
	 *
	 * @param Other The other interval being compared.
	 * @return true if the intervals are equal, false otherwise.
	 */
	[[nodiscard]] bool operator==(const TInterval& Other) const
	{
		return Min == Other.Min && Max == Other.Max;
	}

	/**
	 * Compare two intervals for inequality.
	 *
	 * @param Other The other interval being compared.
	 * @return true if the intervals are not equal, false otherwise.
	 */
	[[nodiscard]] bool operator!=(const TInterval& Other) const
	{
		return (Min != Other.Min) || (Max != Other.Max);
	}

public:
	
	/**
	 * Computes the size of this interval.
	 *
	 * @return Interval size.
	 */
	[[nodiscard]] ElementType Size() const
	{
		return (Max - Min);
	}

	/**
	 * Whether interval is valid (Min <= Max).
	 *
	 * @return false when interval is invalid, true otherwise
	 */
	[[nodiscard]] bool IsValid() const
	{
		return (Min <= Max);
	}
	
	/**
	 * Checks whether this interval contains the specified element.
	 *
	 * @param Element The element to check.
	 * @return true if the range interval the element, false otherwise.
	 */
	[[nodiscard]] bool Contains( const ElementType& Element ) const
	{
		return IsValid() && (Element >= Min && Element <= Max);
	}

	/**
	 * Expands this interval to both sides by the specified amount.
	 *
	 * @param ExpandAmount The amount to expand by.
	 */
	void Expand( ElementType ExpandAmount )
	{
		if (IsValid())
		{
			Min -= ExpandAmount;
			Max += ExpandAmount;
		}
	}

	/**
	 * Expands this interval if necessary to include the specified element.
	 *
	 * @param X The element to include.
	 */
	void Include( ElementType X )
	{
		if (!IsValid())
		{
			Min = X;
			Max = X;
		}
		else
		{
			if (X < Min)
			{
				Min = X;
			}

			if (X > Max)
			{
				Max = X;
			}
		}
	}

	/**
	 * Interval interpolation
	 *
	 * @param Alpha interpolation amount
	 * @return interpolation result
	 */
	[[nodiscard]] ElementType Interpolate( float Alpha ) const
	{
		if (IsValid())
		{
			return Min + ElementType(Alpha*Size());
		}
		
		return ElementType();
	}

	/**
	 * Clamps X to be between the interval inclusively.
	 *
	 * @param X the element to clamp
	 * @return zero if invalid(Min > Max), clamped result otherwise
	 */
	[[nodiscard]] ElementType Clamp( ElementType X ) const
	{
		if (!IsValid())
		{
			return ElementType();
		}

		return FMath::Clamp(X, Min, Max);
	}
	
	/**
	 * Calculate the Percentage of X in the Interval
	 * @param X the element to calculate the percentage
	 * @return zero if invalid(Min > Max), percentage otherwise
	 */
	[[nodiscard]] ElementType GetRangePct(ElementType X) const
	{
		if (!IsValid())
		{
			return ElementType();
		}

		return FMath::GetRangePct(Min, Max, X);
	}
public:

	/**
	 * Calculates the intersection of two intervals.
	 *
	 * @param A The first interval.
	 * @param B The second interval.
	 * @return The intersection.
	 */
	[[nodiscard]] friend TInterval Intersect( const TInterval& A, const TInterval& B )
	{
		if (A.IsValid() && B.IsValid())
		{
			return TInterval(FMath::Max(A.Min, B.Min), FMath::Min(A.Max, B.Max));
		}

		return TInterval();
	}

	/**
	 * Serializes the interval.
	 *
	 * @param Ar The archive to serialize into.
	 * @param Interval The interval to serialize.
	 * @return Reference to the Archive after serialization.
	 */
	friend class FArchive& operator<<( class FArchive& Ar, TInterval& Interval )
	{
		return Ar << Interval.Min << Interval.Max;
	}
	
	/**
	 * Gets the hash for the specified interval.
	 *
	 * @param Interval The Interval to get the hash for.
	 * @return Hash value.
	 */
	friend uint32 GetTypeHash(const TInterval& Interval)
	{
		return HashCombine(GetTypeHash(Interval.Min), GetTypeHash(Interval.Max));
	}
};

/* Default intervals for built-in types
 *****************************************************************************/

#define DEFINE_INTERVAL_WRAPPER_STRUCT(Name, ElementType) \
	struct Name : TInterval<ElementType> \
	{ \
	private: \
		typedef TInterval<ElementType> Super; \
		 \
	public:  \
		Name() \
			: Super() \
		{ \
		} \
		 \
		Name( const Super& Other ) \
			: Super( Other ) \
		{ \
		} \
		 \
		Name( ElementType InMin, ElementType InMax ) \
			: Super( InMin, InMax ) \
		{ \
		} \
		 \
		friend Name Intersect( const Name& A, const Name& B ) \
		{ \
			return Intersect( static_cast<const Super&>( A ), static_cast<const Super&>( B ) ); \
		} \
	}; \
	 \
	template <> \
	struct TIsBitwiseConstructible<Name, TInterval<ElementType>> \
	{ \
		enum { Value = true }; \
	}; \
	 \
	template <> \
	struct TIsBitwiseConstructible<TInterval<ElementType>, Name> \
	{ \
		enum { Value = true }; \
	};

DEFINE_INTERVAL_WRAPPER_STRUCT(FFloatInterval, float)
DEFINE_INTERVAL_WRAPPER_STRUCT(FDoubleInterval, double)
DEFINE_INTERVAL_WRAPPER_STRUCT(FInt32Interval, int32)
