// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Plane.h"
#include "Math/Matrix.h"

namespace UE {
namespace Math {
 	
/** Inverse Rotation matrix */
template<typename T>
struct TInverseRotationMatrix
	: public TMatrix<T>
{
public:
	/**
	 * Constructor.
	 *
	 * @param Rot rotation
	 */
	TInverseRotationMatrix(const TRotator<T>& Rot);
	
	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TInverseRotationMatrix(const TInverseRotationMatrix<FArg>& From)
		: TMatrix<T>(From)
	{
	}
};

template<typename T>
inline TInverseRotationMatrix<T>::TInverseRotationMatrix(const TRotator<T>& Rot)
	: TMatrix<T>(
		TMatrix<T>( // Yaw
			TPlane<T>(+FMath::Cos(Rot.Yaw * UE_PI / 180.f), -FMath::Sin(Rot.Yaw * UE_PI / 180.f), 0.0f, 0.0f),
			TPlane<T>(+FMath::Sin(Rot.Yaw * UE_PI / 180.f), +FMath::Cos(Rot.Yaw * UE_PI / 180.f), 0.0f, 0.0f),
			TPlane<T>(0.0f, 0.0f, 1.0f, 0.0f),
			TPlane<T>(0.0f, 0.0f, 0.0f, 1.0f)) *
		TMatrix<T>( // Pitch
			TPlane<T>(+FMath::Cos(Rot.Pitch * UE_PI / 180.f), 0.0f, -FMath::Sin(Rot.Pitch * UE_PI / 180.f), 0.0f),
			TPlane<T>(0.0f, 1.0f, 0.0f, 0.0f),
			TPlane<T>(+FMath::Sin(Rot.Pitch * UE_PI / 180.f), 0.0f, +FMath::Cos(Rot.Pitch * UE_PI / 180.f), 0.0f),
			TPlane<T>(0.0f, 0.0f, 0.0f, 1.0f)) *
		TMatrix<T>( // Roll
			TPlane<T>(1.0f, 0.0f, 0.0f, 0.0f),
			TPlane<T>(0.0f, +FMath::Cos(Rot.Roll * UE_PI / 180.f), +FMath::Sin(Rot.Roll * UE_PI / 180.f), 0.0f),
			TPlane<T>(0.0f, -FMath::Sin(Rot.Roll * UE_PI / 180.f), +FMath::Cos(Rot.Roll * UE_PI / 180.f), 0.0f),
			TPlane<T>(0.0f, 0.0f, 0.0f, 1.0f))
	)
{ }

} // namespace Math
} // namespace UE

UE_DECLARE_LWC_TYPE(InverseRotationMatrix, 44);

template<> struct TIsUECoreVariant<FInverseRotationMatrix44f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FInverseRotationMatrix44d> { enum { Value = true }; };