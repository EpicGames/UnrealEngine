// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"

namespace UE
{

namespace Math
{

// See https://en.wikipedia.org/wiki/Linear-feedback_shift_register
// This implements maximal-length feedback polynomials LFSR for N bits in [2, 12].
class FLinearFeedbackShiftRegister
{
public:

	[[nodiscard]] FLinearFeedbackShiftRegister() {}

	// LFSR numbers usually only include N^2-1 numbers for N bit. 
	// We do apply a -1 to get the value 0 out.
	// So, for N=4, it should return {0, 1, 2} in random order.
	[[nodiscard]] uint32 GetNextValue(uint32 N)
	{
		return GetNextValueInternal(N, /*IncludeLast*/false);
	}

	// This function is the same as above.
	// But it will also return the last value N^2-1. 
	// So, for N=4, it will return {0, 1, 2, 3} in random order.
	[[nodiscard]] uint32 GetNextValueWithLast(uint32 N)
	{
		return GetNextValueInternal(N, /*IncludeLast*/true);
	}

private:

	const uint32 StartState = 1;
	uint32 State = StartState;

	uint32 GetNextValueInternal(uint32 N, bool IncludeLast)
	{
		// TODO LFRS code could be put in its own h/cpp for encapsulation and reusability
		auto CalcLFSRValue = [&](uint32 Taps0, uint32 Taps1, uint32 Taps2, uint32 Taps3, uint32 Mask)
			{
				if (IncludeLast && State == 0)
				{
					// LFSR loops over 2^m-1 values excluding 0. The returned value has -1 apply to get 0. So we need to generate the last 2^N-1 value. 
					// We use LFSRCache==0 to detect that case and return the last mask value being 0.
					State = StartState;
					return Mask;
				}

				// Update the cach value.
				uint32 Tap = ((Taps0 & State) == 0 ? 0 : 1) ^ ((Taps1 & State) == 0 ? 0 : 1);
				if (Taps2 > 0)
				{
					Tap ^= ((Taps2 & State) == 0 ? 0 : 1);
				}
				if (Taps3 > 0)
				{
					Tap ^= ((Taps3 & State) == 0 ? 0 : 1);
				}
				State = ((State << 1) | Tap) & Mask;

				uint32 ValueToReturn = State - 1;
				if (IncludeLast && State == StartState)
				{
					State = 0;
				}

				return ValueToReturn % Mask;
			};

		check(N >= 2 && N <=12);

		uint32 NewValue = 0;
		if (N == 2)	// 2 bits
		{
			NewValue = CalcLFSRValue(1 << 1, 1 << 0, 0, 0, 0x3);
		}
		else if (N == 3)
		{
			NewValue = CalcLFSRValue(1 << 2, 1 << 1, 0, 0, 0x7);
		}
		else if (N == 4)
		{
			NewValue = CalcLFSRValue(1 << 3, 1 << 2, 0, 0, 0xF);
		}
		else if (N == 5)
		{
			NewValue = CalcLFSRValue(1 << 4, 1 << 2, 0, 0, 0x1F);
		}
		else if (N == 6)
		{
			NewValue = CalcLFSRValue(1 << 5, 1 << 4, 0, 0, 0x3F);
		}
		else if (N == 7)
		{
			NewValue = CalcLFSRValue(1 << 6, 1 << 5, 0, 0, 0x7F);
		}
		else if (N == 8)
		{
			NewValue = CalcLFSRValue(1 << 7, 1 << 5, 1 << 4, 1 << 3, 0xFF);
		}
		else if (N == 9)
		{
			NewValue = CalcLFSRValue(1 << 8, 1 << 4, 0, 0, 0x1FF);
		}
		else if (N == 10)
		{
			NewValue = CalcLFSRValue(1 << 9, 1 << 6, 0, 0, 0x3FF);
		}
		else if (N == 11)
		{
			NewValue = CalcLFSRValue(1 << 10, 1 << 10, 0, 0, 0x7FF);
		}
		else if (N == 12)
		{
			NewValue = CalcLFSRValue(1 << 11, 1 << 10, 1 << 9, 1 << 3, 0xFFF);
		}
		return NewValue;
	}
};

} // namespace UE::Math

} // namespace UE
