// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

struct FColor;
struct FLinearColor;

// Non-macro declaration of the most commonly used types to assist Intellisense.
namespace UE::Math
{
	// Forward declaration of templates
	template<typename T> struct TVector;
	template<typename T> struct TVector2;
	template<typename T> struct TVector4;
	template<typename T> struct TQuat;
	template<typename T> struct TMatrix;
	template<typename T> struct TPlane;
	template<typename T> struct TTransform;
	template<typename T> struct TSphere;
	template<typename T> struct TBox;
	template<typename T> struct TBox2;
	template<typename T> struct TRotator;
	template<typename T, typename TExtent = T> struct TBoxSphereBounds;
	template <typename IntType> struct TIntPoint;
	template <typename IntType> struct TIntRect;
	template <typename IntType> struct TIntVector2;
	template <typename IntType> struct TIntVector3;
	template <typename IntType> struct TIntVector4;
	template<typename T> struct TRay;
}

// ISPC forward declarations
namespace ispc
{
	struct FVector;
	struct FVector2D;
	struct FVector4;
	struct FQuat;
	struct FMatrix;
	struct FPlane;
	struct FTransform;
}

// Forward declaration of concrete types				// Macro version - declares all three variants.
using FVector 		= UE::Math::TVector<double>;		// UE_DECLARE_LWC_TYPE(Vector, 3);
using FVector2D 	= UE::Math::TVector2<double>;		// UE_DECLARE_LWC_TYPE(Vector2,, FVector2D);
using FVector4 		= UE::Math::TVector4<double>;		// UE_DECLARE_LWC_TYPE(Vector4);
using FQuat 		= UE::Math::TQuat<double>;			// UE_DECLARE_LWC_TYPE(Quat, 4);
using FMatrix 		= UE::Math::TMatrix<double>;		// UE_DECLARE_LWC_TYPE(Matrix, 44);
using FPlane 		= UE::Math::TPlane<double>;			// UE_DECLARE_LWC_TYPE(Plane, 4);
using FTransform 	= UE::Math::TTransform<double>;		// UE_DECLARE_LWC_TYPE(Transform, 3);
using FSphere 		= UE::Math::TSphere<double>;		// UE_DECLARE_LWC_TYPE(Sphere, 3);
using FBox 			= UE::Math::TBox<double>;			// UE_DECLARE_LWC_TYPE(Box, 3);
using FBox2D 		= UE::Math::TBox2<double>;			// UE_DECLARE_LWC_TYPE(Box2,, FBox2D);
using FRotator 		= UE::Math::TRotator<double>;		// UE_DECLARE_LWC_TYPE(Rotator, 3);
using FRay			= UE::Math::TRay<double>;			// UE_DECLARE_LWC_TYPE(Ray, 3);

using FVector3d 	= UE::Math::TVector<double>;
using FVector2d 	= UE::Math::TVector2<double>;
using FVector4d 	= UE::Math::TVector4<double>;
using FQuat4d 		= UE::Math::TQuat<double>;
using FMatrix44d 	= UE::Math::TMatrix<double>;
using FPlane4d 		= UE::Math::TPlane<double>;
using FTransform3d 	= UE::Math::TTransform<double>;
using FSphere3d 	= UE::Math::TSphere<double>;
using FBox3d 		= UE::Math::TBox<double>;
using FBox2d 		= UE::Math::TBox2<double>;
using FRotator3d 	= UE::Math::TRotator<double>;
using FRay3d		= UE::Math::TRay<double>;

using FVector3f		= UE::Math::TVector<float>;
using FVector2f		= UE::Math::TVector2<float>;
using FVector4f		= UE::Math::TVector4<float>;
using FQuat4f		= UE::Math::TQuat<float>;
using FMatrix44f	= UE::Math::TMatrix<float>;
using FPlane4f		= UE::Math::TPlane<float>;
using FTransform3f	= UE::Math::TTransform<float>;
using FSphere3f		= UE::Math::TSphere<float>;
using FBox3f		= UE::Math::TBox<float>;
using FBox2f		= UE::Math::TBox2<float>;
using FRotator3f	= UE::Math::TRotator<float>;
using FRay3f		= UE::Math::TRay<float>;


struct FColor;
struct FLinearColor;

// Int vectors
using FIntVector2 = UE::Math::TIntVector2<int32>;
using FIntVector3 = UE::Math::TIntVector3<int32>;
using FIntVector4 = UE::Math::TIntVector4<int32>;
using FInt32Vector2 = UE::Math::TIntVector2<int32>;
using FInt32Vector3 = UE::Math::TIntVector3<int32>;
using FInt32Vector4 = UE::Math::TIntVector4<int32>;
using FInt32Vector = FInt32Vector3;
using FInt64Vector2 = UE::Math::TIntVector2<int64>;
using FInt64Vector3 = UE::Math::TIntVector3<int64>;
using FInt64Vector4 = UE::Math::TIntVector4<int64>;
using FInt64Vector = FInt64Vector3;

using FUintVector2 = UE::Math::TIntVector2<uint32>;
using FUintVector3 = UE::Math::TIntVector3<uint32>;
using FUintVector4 = UE::Math::TIntVector4<uint32>;
using FUint32Vector2 = UE::Math::TIntVector2<uint32>;
using FUint32Vector3 = UE::Math::TIntVector3<uint32>;
using FUint32Vector4 = UE::Math::TIntVector4<uint32>;
using FUint32Vector = FUint32Vector3;
using FUint64Vector2 = UE::Math::TIntVector2<uint64>;
using FUint64Vector3 = UE::Math::TIntVector3<uint64>;
using FUint64Vector4 = UE::Math::TIntVector4<uint64>;
using FUint64Vector = FUint64Vector3;

using FIntVector = FInt32Vector3;
using FUintVector = FUint32Vector3;

// Int points
using FInt32Point = UE::Math::TIntPoint<int32>;
using FInt64Point = UE::Math::TIntPoint<int64>;
using FUint32Point = UE::Math::TIntPoint<uint32>;
using FUint64Point = UE::Math::TIntPoint<uint64>;

using FIntPoint = FInt32Point;
using FUintPoint = FUint32Point;

// Int rects
using FInt32Rect = UE::Math::TIntRect<int32>;
using FInt64Rect = UE::Math::TIntRect<int64>;
using FUint32Rect = UE::Math::TIntRect<uint32>;
using FUint64Rect = UE::Math::TIntRect<uint64>;

using FIntRect = UE::Math::TIntRect<int32>;
using FUintRect = UE::Math::TIntRect<uint32>;


using FBoxSphereBounds3f = UE::Math::TBoxSphereBounds<float, float>;
using FBoxSphereBounds3d = UE::Math::TBoxSphereBounds<double, double>;
// FCompactBoxSphereBounds always stores float extents
using FCompactBoxSphereBounds3d = UE::Math::TBoxSphereBounds<double, float>;

using FBoxSphereBounds = FBoxSphereBounds3d;
using FCompactBoxSphereBounds = FCompactBoxSphereBounds3d;