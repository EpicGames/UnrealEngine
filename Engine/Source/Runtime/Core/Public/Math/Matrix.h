// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Math/MathFwd.h" // IWYU pragma: export
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Math/Plane.h"
#include "Math/Rotator.h"
#include "Math/Axis.h"
#include "Misc/LargeWorldCoordinatesSerializer.h"
#include "UObject/ObjectVersion.h"
#include <type_traits>

UE_DECLARE_LWC_TYPE(Quat, 4);

#ifdef _MSC_VER
#pragma warning (push)
// Ensure template functions don't generate shadowing warnings against global variables at the point of instantiation.
#pragma warning (disable : 4459)
#endif

/**
 * 4x4 matrix of floating point values.
 * 
 * Note that, like with FTransform, Matrix-Matrix multiplication is applied such that
 * C = A * B will yield a transform C that logically first applies A then B,
 * so (A*B).TransformPosition(Pt) == B.TransformPosition(A.TransformPosition(Pt))
 * 
 * Matrix elements are accessed with M[RowIndex][ColumnIndex].
 */
namespace UE
{
namespace Math
{

template<typename T>
struct alignas(16) TMatrix
{
	static_assert(std::is_floating_point_v<T>, "T must be floating point");

public:
	using FReal = T;

	alignas(16) T M[4][4];


	CORE_API static const TMatrix Identity;


#if ENABLE_NAN_DIAGNOSTIC
	inline void DiagnosticCheckNaN() const
	{
		if (ContainsNaN())
		{
			logOrEnsureNanError(TEXT("FMatrix contains NaN: %s"), *ToString());
		}
	}
#else
	void DiagnosticCheckNaN() const {}
#endif


	// Constructors.
	[[nodiscard]] TMatrix() = default;

	/**
	 * Constructor.
	 *
	 * @param EForceInit Force Init Enum.
	 */
	[[nodiscard]] explicit TMatrix(EForceInit)
	{
		FMemory::Memzero(this, sizeof(*this));
	}

	/**
	 * Constructor.
	 *
	 * @param InX X plane
	 * @param InY Y plane
	 * @param InZ Z plane
	 * @param InW W plane
	 */
	[[nodiscard]] TMatrix(const TPlane<T>& InX, const TPlane<T>& InY, const TPlane<T>& InZ, const TPlane<T>& InW);

	/**
	 * Constructor.
	 *
	 * @param InX X vector
	 * @param InY Y vector
	 * @param InZ Z vector
	 * @param InW W vector
	 */
	[[nodiscard]] TMatrix(const TVector<T>& InX, const TVector<T>& InY, const TVector<T>& InZ, const TVector<T>& InW);

	// Set this to the identity matrix
	inline void SetIdentity();

	/**
	 * Gets the result of multiplying a Matrix to this.
	 *
	 * @param Other The matrix to multiply this by.
	 * @return The result of multiplication.
	 */
	[[nodiscard]] TMatrix<T> operator* (const TMatrix<T>& Other) const;

	/**
	 * Multiply this by a matrix.
	 *
	 * @param Other the matrix to multiply by this.
	 * @return reference to this after multiply.
	 */
	void operator*=(const TMatrix<T>& Other);

	/**
	 * Gets the result of adding a matrix to this.
	 *
	 * @param Other The Matrix to add.
	 * @return The result of addition.
	 */
	[[nodiscard]] TMatrix<T> operator+ (const TMatrix<T>& Other) const;

	/**
	 * Adds to this matrix.
	 *
	 * @param Other The matrix to add to this.
	 * @return Reference to this after addition.
	 */
	void operator+=(const TMatrix<T>& Other);

	/**
	  * This isn't applying SCALE, just multiplying the value to all members - i.e. weighting
	  */
	[[nodiscard]] TMatrix<T> operator* (T Other) const;

	/**
	 * Multiply this matrix by a weighting factor.
	 *
	 * @param other The weight.
	 * @return a reference to this after weighting.
	 */
	void operator*=(T Other);

	/**
	 * Checks whether two matrix are identical.
	 *
	 * @param Other The other matrix.
	 * @return true if two matrix are identical, otherwise false.
	 */
	[[nodiscard]] inline bool operator==(const TMatrix<T>& Other) const;

	/**
	 * Checks whether another Matrix is equal to this, within specified tolerance.
	 *
	 * @param Other The other Matrix.
	 * @param Tolerance Error Tolerance.
	 * @return true if two Matrix are equal, within specified tolerance, otherwise false.
	 */
	[[nodiscard]] inline bool Equals(const TMatrix<T>& Other, T Tolerance = UE_KINDA_SMALL_NUMBER) const;

	/**
	 * Checks whether another Matrix is not equal to this, within specified tolerance.
	 *
	 * @param Other The other Matrix.
	 * @return true if two Matrix are not equal, within specified tolerance, otherwise false.
	 */
	[[nodiscard]] inline bool operator!=(const TMatrix<T>& Other) const;

	// Homogeneous transform.
	[[nodiscard]] TVector4<T> TransformFVector4(const TVector4<T>& V) const;

	/** Transform a location - will take into account translation part of the TMatrix<T>. */
	[[nodiscard]] TVector4<T> TransformPosition(const TVector<T>& V) const;

	/** Inverts the matrix and then transforms V - correctly handles scaling in this matrix. */
	[[nodiscard]] TVector<T> InverseTransformPosition(const TVector<T>& V) const;

	/**
	 *	Transform a direction vector - will not take into account translation part of the TMatrix<T>.
	 *	If you want to transform a surface normal (or plane) and correctly account for non-uniform scaling you should use TransformByUsingAdjointT.
	 */
	[[nodiscard]] TVector4<T> TransformVector(const TVector<T>& V) const;

	/**
	 *	Transform a direction vector by the inverse of this matrix - will not take into account translation part.
	 *	If you want to transform a surface normal (or plane) and correctly account for non-uniform scaling you should use TransformByUsingAdjointT with adjoint of matrix inverse.
	 */
	[[nodiscard]] TVector<T> InverseTransformVector(const TVector<T>& V) const;

	// Transpose.

	[[nodiscard]] TMatrix<T> GetTransposed() const;

	// @return determinant of this matrix.

	[[nodiscard]] inline T Determinant() const;

	/** @return the determinant of rotation 3x3 matrix */
	[[nodiscard]] inline T RotDeterminant() const;

	/** Get the inverse of this matrix.  Will ensure on nil matrices in non-final builds.  Not faster than Inverse. */
	[[nodiscard]] inline TMatrix<T> InverseFast() const;

	/** Get the inverse of this matrix.  Will silently change nil/nan matrices to identity. */
	[[nodiscard]] inline TMatrix<T> Inverse() const;

	[[nodiscard]] inline TMatrix<T> TransposeAdjoint() const;

	// NOTE: There is some compiler optimization issues with WIN64 that cause FORCEINLINE to cause a crash
	// Remove any scaling from this matrix (ie magnitude of each row is 1) with error Tolerance
	inline void RemoveScaling(T Tolerance = UE_SMALL_NUMBER);

	// Returns matrix after RemoveScaling with error Tolerance
	[[nodiscard]] inline TMatrix<T> GetMatrixWithoutScale(T Tolerance = UE_SMALL_NUMBER) const;

	/** Remove any scaling from this matrix (ie magnitude of each row is 1) and return the 3D scale vector that was initially present with error Tolerance */
	inline TVector<T> ExtractScaling(T Tolerance = UE_SMALL_NUMBER);

	/** return a 3D scale vector calculated from this matrix (where each component is the magnitude of a row vector) with error Tolerance. */
	[[nodiscard]] inline TVector<T> GetScaleVector(T Tolerance = UE_SMALL_NUMBER) const;

	// Remove any translation from this matrix
	[[nodiscard]] inline TMatrix<T> RemoveTranslation() const;

	/** Returns a matrix with an additional translation concatenated. */
	[[nodiscard]] inline TMatrix<T> ConcatTranslation(const TVector<T>& Translation) const;

	/** Returns true if any element of this matrix is NaN */
	[[nodiscard]] inline bool ContainsNaN() const;

	/** Scale the translation part of the matrix by the supplied vector. */
	inline void ScaleTranslation(const TVector<T>& Scale3D);

	/** @return the minimum magnitude of any row of the matrix. */
	[[nodiscard]] inline T GetMinimumAxisScale() const;

	/** @return the maximum magnitude of any row of the matrix. */
	[[nodiscard]] inline T GetMaximumAxisScale() const;

	/** Apply Scale to this matrix **/
	[[nodiscard]] inline TMatrix<T> ApplyScale(T Scale) const;

	// @return the origin of the co-ordinate system
	[[nodiscard]] inline TVector<T> GetOrigin() const;

	/**
	 * get axis of this matrix scaled by the scale of the matrix
	 *
	 * @param i index into the axis of the matrix
	 * @ return vector of the axis
	 */
	[[nodiscard]] inline TVector<T> GetScaledAxis(EAxis::Type Axis) const;

	/**
	 * get axes of this matrix scaled by the scale of the matrix
	 *
	 * @param X axes returned to this param
	 * @param Y axes returned to this param
	 * @param Z axes returned to this param
	 */
	inline void GetScaledAxes(TVector<T>& X, TVector<T>& Y, TVector<T>& Z) const;

	/**
	 * get unit length axis of this matrix
	 *
	 * @param i index into the axis of the matrix
	 * @return vector of the axis
	 */
	[[nodiscard]] inline TVector<T> GetUnitAxis(EAxis::Type Axis) const;

	/**
	 * get unit length axes of this matrix
	 *
	 * @param X axes returned to this param
	 * @param Y axes returned to this param
	 * @param Z axes returned to this param
	 */
	inline void GetUnitAxes(TVector<T>& X, TVector<T>& Y, TVector<T>& Z) const;

	/**
	 * set an axis of this matrix
	 *
	 * @param i index into the axis of the matrix
	 * @param Axis vector of the axis
	 */
	inline void SetAxis(int32 i, const TVector<T>& Axis);

	// Set the origin of the coordinate system to the given vector
	inline void SetOrigin(const TVector<T>& NewOrigin);

	/**
	 * Update the axes of the matrix if any value is NULL do not update that axis
	 *
	 * @param Axis0 set matrix row 0
	 * @param Axis1 set matrix row 1
	 * @param Axis2 set matrix row 2
	 * @param Origin set matrix row 3
	 */
	inline void SetAxes(const TVector<T>* Axis0 = NULL, const TVector<T>* Axis1 = NULL, const TVector<T>* Axis2 = NULL, const TVector<T>* Origin = NULL);

	/**
	 * get a column of this matrix
	 *
	 * @param i index into the column of the matrix
	 * @return vector of the column
	 */
	[[nodiscard]] inline TVector<T> GetColumn(int32 i) const;

	/**
	 * Set a column of this matrix
	 *
	 * @param i index of the matrix column
	 * @param Value new value of the column
	 */
	inline void SetColumn(int32 i, TVector<T> Value);

	/** @return rotator representation of this matrix */
	[[nodiscard]] CORE_API UE::Math::TRotator<T> Rotator() const;

	/**
	 * Transform a rotation matrix into a quaternion.
	 *
	 * @warning rotation part will need to be unit length for this to be right!
	 */
	[[nodiscard]] CORE_API UE::Math::TQuat<T> ToQuat() const;

	/**
	 * Convert this Atom to the 3x4 transpose of the transformation matrix.
	 */
	void To3x4MatrixTranspose(T* Out) const;

	// Frustum plane extraction.
	/** @param OuTPln the near plane of the Frustum of this matrix */
	bool GetFrustumNearPlane(TPlane<T>& OuTPln) const;

	/** @param OuTPln the far plane of the Frustum of this matrix */
	bool GetFrustumFarPlane(TPlane<T>& OuTPln) const;

	/** @param OuTPln the left plane of the Frustum of this matrix */
	bool GetFrustumLeftPlane(TPlane<T>& OuTPln) const;

	/** @param OuTPln the right plane of the Frustum of this matrix */
	bool GetFrustumRightPlane(TPlane<T>& OuTPln) const;

	/** @param OuTPln the top plane of the Frustum of this matrix */
	bool GetFrustumTopPlane(TPlane<T>& OuTPln) const;

	/** @param OuTPln the bottom plane of the Frustum of this matrix */
	bool GetFrustumBottomPlane(TPlane<T>& OuTPln) const;

	/**
	 * Utility for mirroring this transform across a certain plane, and flipping one of the axis as well.
	 */
	inline void Mirror(EAxis::Type MirrorAxis, EAxis::Type FlipAxis);

	/**
	 * Get a textual representation of the vector.
	 *
	 * @return Text describing the vector.
	 */
	[[nodiscard]] FString ToString() const
	{
		FString Output;

		Output += FString::Printf(TEXT("[%g %g %g %g] "), M[0][0], M[0][1], M[0][2], M[0][3]);
		Output += FString::Printf(TEXT("[%g %g %g %g] "), M[1][0], M[1][1], M[1][2], M[1][3]);
		Output += FString::Printf(TEXT("[%g %g %g %g] "), M[2][0], M[2][1], M[2][2], M[2][3]);
		Output += FString::Printf(TEXT("[%g %g %g %g] "), M[3][0], M[3][1], M[3][2], M[3][3]);

		return Output;
	}

	/** Output ToString */
	void DebugPrint() const
	{
		UE_LOGF(LogUnrealMath, Log, "%ls", *ToString());
	}

	/** For debugging purpose, could be changed */
	[[nodiscard]] uint32 ComputeHash() const
	{
		uint32 Ret = 0;

		const uint32* Data = (uint32*)this;

		for (uint32 i = 0; i < 16; ++i)
		{
			Ret ^= Data[i] + i;
		}

		return Ret;
	}

	bool Serialize(FArchive& Ar)
	{
		//if (Ar.UEVer() >= VER_UE4_ADDED_NATIVE_SERIALIZATION_FOR_IMMUTABLE_STRUCTURES)
		{
			Ar << (TMatrix<T>&)*this;
			return true;
		}
		//return false;
	}
	
	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		if constexpr (std::is_same_v<T, float>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Matrix, Matrix44f, Matrix44d);
		}
		else
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Matrix, Matrix44d, Matrix44f);
		}
	}

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TMatrix(const TMatrix<FArg>& From)
	{
#if PLATFORM_ENABLE_VECTORINTRINSICS
		const FArg* RESTRICT Src = &(From.M[0][0]);
		T* RESTRICT Dest = &(M[0][0]);

		// Load src
		TVectorRegisterType<FArg> SrcRow0 = VectorLoad(&Src[0]);
		TVectorRegisterType<FArg> SrcRow1 = VectorLoad(&Src[4]);
		TVectorRegisterType<FArg> SrcRow2 = VectorLoad(&Src[8]);
		TVectorRegisterType<FArg> SrcRow3 = VectorLoad(&Src[12]);

		// Coerce and store
		if constexpr (std::is_same_v<T, float>)
		{
			VectorStore(MakeVectorRegisterFloatFromDouble(SrcRow0), &Dest[0]);
			VectorStore(MakeVectorRegisterFloatFromDouble(SrcRow1), &Dest[4]);
			VectorStore(MakeVectorRegisterFloatFromDouble(SrcRow2), &Dest[8]);
			VectorStore(MakeVectorRegisterFloatFromDouble(SrcRow3), &Dest[12]);
		}
		else
		{
			VectorStore(TVectorRegisterType<T>(SrcRow0), &Dest[0]);
			VectorStore(TVectorRegisterType<T>(SrcRow1), &Dest[4]);
			VectorStore(TVectorRegisterType<T>(SrcRow2), &Dest[8]);
			VectorStore(TVectorRegisterType<T>(SrcRow3), &Dest[12]);
		}
#else
		M[0][0] = (T)From.M[0][0]; M[0][1] = (T)From.M[0][1]; M[0][2] = (T)From.M[0][2]; M[0][3] = (T)From.M[0][3];
		M[1][0] = (T)From.M[1][0]; M[1][1] = (T)From.M[1][1]; M[1][2] = (T)From.M[1][2]; M[1][3] = (T)From.M[1][3];
		M[2][0] = (T)From.M[2][0]; M[2][1] = (T)From.M[2][1]; M[2][2] = (T)From.M[2][2]; M[2][3] = (T)From.M[2][3];
		M[3][0] = (T)From.M[3][0]; M[3][1] = (T)From.M[3][1]; M[3][2] = (T)From.M[3][2]; M[3][3] = (T)From.M[3][3];
#endif

		DiagnosticCheckNaN();
	}

private:

	/**
	 * Output an error message and trigger an ensure
	 */
	static void ErrorEnsure(const TCHAR* Message)
	{
		UE_LOGF(LogUnrealMath, Error, "%ls", Message);
		ensureMsgf(false, TEXT("%s"), Message);
	}
};

#if !defined(_MSC_VER) || defined(__clang__) // MSVC can't forward declare explicit specializations
template<> CORE_API const FMatrix44f FMatrix44f::Identity;
template<> CORE_API const FMatrix44d FMatrix44d::Identity;
#endif


/**
 * Serializes the Matrix.
 *
 * @param Ar Reference to the serialization archive.
 * @param M Reference to the matrix being serialized.
 * @return Reference to the Archive after serialization.
 */
inline FArchive& operator<<(FArchive& Ar, TMatrix<float>& M)
{
	Ar << M.M[0][0] << M.M[0][1] << M.M[0][2] << M.M[0][3];
	Ar << M.M[1][0] << M.M[1][1] << M.M[1][2] << M.M[1][3];
	Ar << M.M[2][0] << M.M[2][1] << M.M[2][2] << M.M[2][3];
	Ar << M.M[3][0] << M.M[3][1] << M.M[3][2] << M.M[3][3];
	M.DiagnosticCheckNaN();
	return Ar;
}

/**
 * Serializes the Matrix.
 *
 * @param Ar Reference to the serialization archive.
 * @param M Reference to the matrix being serialized.
 * @return Reference to the Archive after serialization.
 */
inline FArchive& operator<<(FArchive& Ar, TMatrix<double>& M)
{
	if (Ar.UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
	{
		Ar << M.M[0][0] << M.M[0][1] << M.M[0][2] << M.M[0][3];
		Ar << M.M[1][0] << M.M[1][1] << M.M[1][2] << M.M[1][3];
		Ar << M.M[2][0] << M.M[2][1] << M.M[2][2] << M.M[2][3];
		Ar << M.M[3][0] << M.M[3][1] << M.M[3][2] << M.M[3][3];
	}
	else
	{
		checkf(Ar.IsLoading(), TEXT("float -> double conversion applied outside of load!"));
		// Stored as floats, so serialize float and copy.
		for (int32 Row = 0; Row < 4; ++Row)
		{
			float Col0, Col1, Col2, Col3;
			Ar << Col0 << Col1 << Col2 << Col3;
			M.M[Row][0] = Col0;
			M.M[Row][1] = Col1;
			M.M[Row][2] = Col2;
			M.M[Row][3] = Col3;
		}
	}
	M.DiagnosticCheckNaN();
	return Ar;
}

template<typename T>
struct TBasisVectorMatrix : public TMatrix<T>
{
	using TMatrix<T>::M;

	// Create Basis matrix from 3 axis vectors and the origin
	TBasisVectorMatrix(const TVector<T>& XAxis,const TVector<T>& YAxis,const TVector<T>& ZAxis,const TVector<T>& Origin);

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TBasisVectorMatrix(const TBasisVectorMatrix<FArg>& From)
		: TMatrix<T>(From)
	{
	}
};


template<typename T>
struct TLookFromMatrix : public TMatrix<T>
{
	using TMatrix<T>::M;

	/**
	* Creates a view matrix given an eye position, a direction to look in, and an up vector.
	* Direction or up vectors need not be normalized.
	* This does the same thing as FLookAtMatrix, except without completely destroying precision when position is large,
	* Always use this instead of e.g., FLookAtMatrix(Pos, Pos + Dir,...);
	*/
	TLookFromMatrix(const TVector<T>& EyePosition, const TVector<T>& LookDirection, const TVector<T>& UpVector);

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TLookFromMatrix(const TLookFromMatrix<FArg>& From)
		: TMatrix<T>(From)
	{
	}
};


template<typename T>
struct TLookAtMatrix : public TLookFromMatrix<T>
{
	using TLookFromMatrix<T>::M;

	/** 
	* Creates a view matrix given an eye position, a position to look at, and an up vector. 
	* Equivalent of FLookFromMatrix(EyePosition, LookAtPosition - EyePosition, UpVector)
	* The up vector need not be normalized.
	* This does the same thing as D3DXMatrixLookAtLH.
	*/
	TLookAtMatrix(const TVector<T>& EyePosition, const TVector<T>& LookAtPosition, const TVector<T>& UpVector);

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TLookAtMatrix(const TLookAtMatrix<FArg>& From)
		: TLookFromMatrix<T>(From)
	{
	}
};

} // namespace UE::Math
} // namespace UE

UE_DECLARE_LWC_TYPE(Matrix, 44);
UE_DECLARE_LWC_TYPE(BasisVectorMatrix, 44);
UE_DECLARE_LWC_TYPE(LookFromMatrix, 44);
UE_DECLARE_LWC_TYPE(LookAtMatrix, 44);

#define UE_DECLARE_MATRIX_TYPE_TRAITS(TYPE)										\
template<> struct TIsPODType<F##TYPE##44f> { enum { Value = true }; };			\
template<> struct TIsUECoreVariant<F##TYPE##44f> { enum { Value = true }; };	\
template<> struct TIsPODType<F##TYPE##44d> { enum { Value = true }; };			\
template<> struct TIsUECoreVariant<F##TYPE##44d> { enum { Value = true }; };	\

UE_DECLARE_MATRIX_TYPE_TRAITS(Matrix);
UE_DECLARE_MATRIX_TYPE_TRAITS(BasisVectorMatrix);
UE_DECLARE_MATRIX_TYPE_TRAITS(LookFromMatrix);
UE_DECLARE_MATRIX_TYPE_TRAITS(LookAtMatrix);

#undef UE_DECLARE_MATRIX_TYPE_TRAITS

// Forward declare all explicit specializations (in UnrealMath.cpp)
template<> CORE_API FQuat4f FMatrix44f::ToQuat() const;
template<> CORE_API FQuat4d FMatrix44d::ToQuat() const;


// very high quality 4x4 matrix inverse
// @todo: this is redundant with FMatrix44d::Inverse and should be removed ; seems to be unused
template<typename FArg UE_REQUIRES(std::is_floating_point_v<FArg>)>
static inline bool Inverse4x4( double* dst, const FArg* src )
{
	const double s0  = (double)(src[ 0]); const double s1  = (double)(src[ 1]); const double s2  = (double)(src[ 2]); const double s3  = (double)(src[ 3]);
	const double s4  = (double)(src[ 4]); const double s5  = (double)(src[ 5]); const double s6  = (double)(src[ 6]); const double s7  = (double)(src[ 7]);
	const double s8  = (double)(src[ 8]); const double s9  = (double)(src[ 9]); const double s10 = (double)(src[10]); const double s11 = (double)(src[11]);
	const double s12 = (double)(src[12]); const double s13 = (double)(src[13]); const double s14 = (double)(src[14]); const double s15 = (double)(src[15]);

	double inv[16];
	inv[0]  =  s5 * s10 * s15 - s5 * s11 * s14 - s9 * s6 * s15 + s9 * s7 * s14 + s13 * s6 * s11 - s13 * s7 * s10;
	inv[1]  = -s1 * s10 * s15 + s1 * s11 * s14 + s9 * s2 * s15 - s9 * s3 * s14 - s13 * s2 * s11 + s13 * s3 * s10;
	inv[2]  =  s1 * s6  * s15 - s1 * s7  * s14 - s5 * s2 * s15 + s5 * s3 * s14 + s13 * s2 * s7  - s13 * s3 * s6;
	inv[3]  = -s1 * s6  * s11 + s1 * s7  * s10 + s5 * s2 * s11 - s5 * s3 * s10 - s9  * s2 * s7  + s9  * s3 * s6;
	inv[4]  = -s4 * s10 * s15 + s4 * s11 * s14 + s8 * s6 * s15 - s8 * s7 * s14 - s12 * s6 * s11 + s12 * s7 * s10;
	inv[5]  =  s0 * s10 * s15 - s0 * s11 * s14 - s8 * s2 * s15 + s8 * s3 * s14 + s12 * s2 * s11 - s12 * s3 * s10;
	inv[6]  = -s0 * s6  * s15 + s0 * s7  * s14 + s4 * s2 * s15 - s4 * s3 * s14 - s12 * s2 * s7  + s12 * s3 * s6;
	inv[7]  =  s0 * s6  * s11 - s0 * s7  * s10 - s4 * s2 * s11 + s4 * s3 * s10 + s8  * s2 * s7  - s8  * s3 * s6;
	inv[8]  =  s4 * s9  * s15 - s4 * s11 * s13 - s8 * s5 * s15 + s8 * s7 * s13 + s12 * s5 * s11 - s12 * s7 * s9;
	inv[9]  = -s0 * s9  * s15 + s0 * s11 * s13 + s8 * s1 * s15 - s8 * s3 * s13 - s12 * s1 * s11 + s12 * s3 * s9;
	inv[10] =  s0 * s5  * s15 - s0 * s7  * s13 - s4 * s1 * s15 + s4 * s3 * s13 + s12 * s1 * s7  - s12 * s3 * s5;
	inv[11] = -s0 * s5  * s11 + s0 * s7  * s9  + s4 * s1 * s11 - s4 * s3 * s9  - s8  * s1 * s7  + s8  * s3 * s5;
	inv[12] = -s4 * s9  * s14 + s4 * s10 * s13 + s8 * s5 * s14 - s8 * s6 * s13 - s12 * s5 * s10 + s12 * s6 * s9;
	inv[13] =  s0 * s9  * s14 - s0 * s10 * s13 - s8 * s1 * s14 + s8 * s2 * s13 + s12 * s1 * s10 - s12 * s2 * s9;
	inv[14] = -s0 * s5  * s14 + s0 * s6  * s13 + s4 * s1 * s14 - s4 * s2 * s13 - s12 * s1 * s6  + s12 * s2 * s5;
	inv[15] =  s0 * s5  * s10 - s0 * s6  * s9  - s4 * s1 * s10 + s4 * s2 * s9  + s8  * s1 * s6  - s8  * s2 * s5;

	double det = s0 * inv[0] + s1 * inv[4] + s2 * inv[8] + s3 * inv[12];
	if( det == 0.0 || !FMath::IsFinite(det) )
	{
		memcpy(dst,&FMatrix44d::Identity,sizeof(FMatrix44d));
		return false;
	}

	det = 1.0 / det;
	for( int i = 0; i < 16; i++ )
	{
		dst[i] = inv[i] * det;
	}
	return true;
}

#include "Math/Matrix.inl" // IWYU pragma: export

#ifdef _MSC_VER
#pragma warning (pop)
#endif
