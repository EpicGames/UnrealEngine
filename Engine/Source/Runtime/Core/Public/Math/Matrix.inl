// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NOTE: This file should ONLY be included by UnrealMath.h!
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "CoreFwd.h"

UE_DECLARE_LWC_TYPE(BasisVectorMatrix, 44);
UE_DECLARE_LWC_TYPE(LookAtMatrix, 44);
struct FMath;

namespace UE
{
namespace Math
{

/**
 * TMatrix inline functions.
 */

 // Constructors.
template<typename T>
inline  TMatrix<T>::TMatrix(const TPlane<T>& InX, const TPlane<T>& InY, const TPlane<T>& InZ, const TPlane<T>& InW)
{
	M[0][0] = InX.X; M[0][1] = InX.Y;  M[0][2] = InX.Z;  M[0][3] = InX.W;
	M[1][0] = InY.X; M[1][1] = InY.Y;  M[1][2] = InY.Z;  M[1][3] = InY.W;
	M[2][0] = InZ.X; M[2][1] = InZ.Y;  M[2][2] = InZ.Z;  M[2][3] = InZ.W;
	M[3][0] = InW.X; M[3][1] = InW.Y;  M[3][2] = InW.Z;  M[3][3] = InW.W;
	DiagnosticCheckNaN();
}

template<typename T>
inline  TMatrix<T>::TMatrix(const TVector<T>& InX, const TVector<T>& InY, const TVector<T>& InZ, const TVector<T>& InW)
{
	M[0][0] = InX.X; M[0][1] = InX.Y;  M[0][2] = InX.Z;  M[0][3] = 0.0f;
	M[1][0] = InY.X; M[1][1] = InY.Y;  M[1][2] = InY.Z;  M[1][3] = 0.0f;
	M[2][0] = InZ.X; M[2][1] = InZ.Y;  M[2][2] = InZ.Z;  M[2][3] = 0.0f;
	M[3][0] = InW.X; M[3][1] = InW.Y;  M[3][2] = InW.Z;  M[3][3] = 1.0f;
	DiagnosticCheckNaN();
}


template<typename T>
inline void  TMatrix<T>::SetIdentity()
{
	M[0][0] = 1; M[0][1] = 0;  M[0][2] = 0;  M[0][3] = 0;
	M[1][0] = 0; M[1][1] = 1;  M[1][2] = 0;  M[1][3] = 0;
	M[2][0] = 0; M[2][1] = 0;  M[2][2] = 1;  M[2][3] = 0;
	M[3][0] = 0; M[3][1] = 0;  M[3][2] = 0;  M[3][3] = 1;
}


template<typename T>
inline void TMatrix<T>::operator*=(const TMatrix<T>& Other)
{
	VectorMatrixMultiply(this, this, &Other);
	DiagnosticCheckNaN();
}


template<typename T>
inline TMatrix<T> TMatrix<T>::operator*(const TMatrix<T>& Other) const
{
	TMatrix<T> Result;
	VectorMatrixMultiply(&Result, this, &Other);
	Result.DiagnosticCheckNaN();
	return Result;
}


template<typename T>
inline TMatrix<T>	TMatrix<T>::operator+(const TMatrix<T>& Other) const
{
	TMatrix<T> ResultMat;

	for (int32 X = 0; X < 4; X++)
	{
		ResultMat.M[X][0] = M[X][0] + Other.M[X][0];
		ResultMat.M[X][1] = M[X][1] + Other.M[X][1];
		ResultMat.M[X][2] = M[X][2] + Other.M[X][2];
		ResultMat.M[X][3] = M[X][3] + Other.M[X][3];
	}

	ResultMat.DiagnosticCheckNaN();
	return ResultMat;
}

template<typename T>
inline void TMatrix<T>::operator+=(const TMatrix<T>& Other)
{
	*this = *this + Other;
	DiagnosticCheckNaN();
}

template<typename T>
inline TMatrix<T> TMatrix<T>::operator*(T Other) const
{
	TMatrix<T> ResultMat;

	for (int32 X = 0; X < 4; X++)
	{
		ResultMat.M[X][0] = M[X][0] * Other;
		ResultMat.M[X][1] = M[X][1] * Other;
		ResultMat.M[X][2] = M[X][2] * Other;
		ResultMat.M[X][3] = M[X][3] * Other;
	}

	ResultMat.DiagnosticCheckNaN();
	return ResultMat;
}

template<typename T>
inline void TMatrix<T>::operator*=(T Other)
{
	*this = *this * Other;
	DiagnosticCheckNaN();
}

// Comparison operators.

template<typename T>
inline bool TMatrix<T>::operator==(const TMatrix<T>& Other) const
{
	for (int32 X = 0; X < 4; X++)
	{
		for (int32 Y = 0; Y < 4; Y++)
		{
			if (M[X][Y] != Other.M[X][Y])
			{
				return false;
			}
		}
	}

	return true;
}

// Error-tolerant comparison.
template<typename T>
inline bool TMatrix<T>::Equals(const TMatrix<T>& Other, T Tolerance/*=KINDA_SMALL_NUMBER*/) const
{
	for (int32 X = 0; X < 4; X++)
	{
		for (int32 Y = 0; Y < 4; Y++)
		{
			if (FMath::Abs(M[X][Y] - Other.M[X][Y]) > Tolerance)
			{
				return false;
			}
		}
	}

	return true;
}

template<typename T>
inline bool TMatrix<T>::operator!=(const TMatrix<T>& Other) const
{
	return !(*this == Other);
}


// Homogeneous transform.

template<typename T>
inline TVector4<T> TMatrix<T>::TransformFVector4(const TVector4<T>& P) const
{
	TVector4<T> Result;
	TVectorRegisterType<T> VecP = VectorLoadAligned(&P);
	TVectorRegisterType<T> VecR = VectorTransformVector(VecP, this);
	VectorStoreAligned(VecR, &Result);
	return Result;
}


// Transform position

/** Transform a location - will take into account translation part of the TMatrix<T>. */
template<typename T>
UE_FORCEINLINE_HINT TVector4<T> TMatrix<T>::TransformPosition(const TVector<T>& V) const
{
	return TransformFVector4(TVector4<T>(V.X, V.Y, V.Z, 1.0f));
}

/** Inverts the matrix and then transforms V - correctly handles scaling in this matrix. */
template<typename T>
inline TVector<T> TMatrix<T>::InverseTransformPosition(const TVector<T>& V) const
{
	TMatrix<T> InvSelf = this->InverseFast();
	return InvSelf.TransformPosition(V);
}

// Transform vector

/**
 *	Transform a direction vector - will not take into account translation part of the TMatrix<T>.
 *	If you want to transform a surface normal (or plane) and correctly account for non-uniform scaling you should use TransformByUsingAdjointT.
 */
template<typename T>
UE_FORCEINLINE_HINT TVector4<T> TMatrix<T>::TransformVector(const TVector<T>& V) const
{
	return TransformFVector4(TVector4<T>(V.X, V.Y, V.Z, 0.0f));
}

/** Faster version of InverseTransformVector that assumes no scaling. WARNING: Will NOT work correctly if there is scaling in the matrix. */
template<typename T>
inline TVector<T> TMatrix<T>::InverseTransformVector(const TVector<T>& V) const
{
	TMatrix<T> InvSelf = this->InverseFast();
	return InvSelf.TransformVector(V);
}


// Transpose.

template<typename T>
inline TMatrix<T> TMatrix<T>::GetTransposed() const
{
	TMatrix<T>	Result;

	Result.M[0][0] = M[0][0];
	Result.M[0][1] = M[1][0];
	Result.M[0][2] = M[2][0];
	Result.M[0][3] = M[3][0];

	Result.M[1][0] = M[0][1];
	Result.M[1][1] = M[1][1];
	Result.M[1][2] = M[2][1];
	Result.M[1][3] = M[3][1];

	Result.M[2][0] = M[0][2];
	Result.M[2][1] = M[1][2];
	Result.M[2][2] = M[2][2];
	Result.M[2][3] = M[3][2];

	Result.M[3][0] = M[0][3];
	Result.M[3][1] = M[1][3];
	Result.M[3][2] = M[2][3];
	Result.M[3][3] = M[3][3];

	return Result;
}

/**
* Convert this Atom to the 3x4 transpose of the transformation matrix.
*/
template<typename T>
inline void TMatrix<T>::To3x4MatrixTranspose(T* Out) const
{
	const T* RESTRICT Src = &(M[0][0]);
	T* RESTRICT Dest = Out;

	Dest[0] = Src[0];   // [0][0]
	Dest[1] = Src[4];   // [1][0]
	Dest[2] = Src[8];   // [2][0]
	Dest[3] = Src[12];  // [3][0]

	Dest[4] = Src[1];   // [0][1]
	Dest[5] = Src[5];   // [1][1]
	Dest[6] = Src[9];   // [2][1]
	Dest[7] = Src[13];  // [3][1]

	Dest[8] = Src[2];   // [0][2]
	Dest[9] = Src[6];   // [1][2]
	Dest[10] = Src[10];  // [2][2]
	Dest[11] = Src[14];  // [3][2]
}

template<>
inline void TMatrix<float>::To3x4MatrixTranspose(float* Out) const
{
	const float* RESTRICT Src = &(M[0][0]);
	float* RESTRICT Dest = Out;

#if PLATFORM_ENABLE_VECTORINTRINSICS
	VectorRegister4Float InRow0 = VectorLoad(&Src[0]);
	VectorRegister4Float InRow1 = VectorLoad(&Src[4]);
	VectorRegister4Float InRow2 = VectorLoad(&Src[8]);
	VectorRegister4Float InRow3 = VectorLoad(&Src[12]);

	VectorRegister4Float Temp0 = VectorCombineLow( InRow0, InRow1);
	VectorRegister4Float Temp1 = VectorCombineLow( InRow2, InRow3);
	VectorRegister4Float Temp2 = VectorCombineHigh(InRow0, InRow1);
	VectorRegister4Float Temp3 = VectorCombineHigh(InRow2, InRow3);

	VectorRegister4Float Row0, Row1, Row2, Row3;
	VectorDeinterleave(Row0, Row1, Temp0, Temp1);
	VectorDeinterleave(Row2, Row3, Temp2, Temp3);

	VectorStore(Row0, &Dest[0]);
	VectorStore(Row1, &Dest[4]);
	VectorStore(Row2, &Dest[8]);
#else
	Dest[0]  = Src[0];   // [0][0]
	Dest[1]  = Src[4];   // [1][0]
	Dest[2]  = Src[8];   // [2][0]
	Dest[3]  = Src[12];  // [3][0]

	Dest[4]  = Src[1];   // [0][1]
	Dest[5]  = Src[5];   // [1][1]
	Dest[6]  = Src[9];   // [2][1]
	Dest[7]  = Src[13];  // [3][1]

	Dest[8]  = Src[2];   // [0][2]
	Dest[9]  = Src[6];   // [1][2]
	Dest[10] = Src[10];  // [2][2]
	Dest[11] = Src[14];  // [3][2]
#endif
}

// Determinant.

template<typename T>
inline T TMatrix<T>::Determinant() const
{
	return	M[0][0] * (
		M[1][1] * (M[2][2] * M[3][3] - M[2][3] * M[3][2]) -
		M[2][1] * (M[1][2] * M[3][3] - M[1][3] * M[3][2]) +
		M[3][1] * (M[1][2] * M[2][3] - M[1][3] * M[2][2])
		) -
		M[1][0] * (
			M[0][1] * (M[2][2] * M[3][3] - M[2][3] * M[3][2]) -
			M[2][1] * (M[0][2] * M[3][3] - M[0][3] * M[3][2]) +
			M[3][1] * (M[0][2] * M[2][3] - M[0][3] * M[2][2])
			) +
		M[2][0] * (
			M[0][1] * (M[1][2] * M[3][3] - M[1][3] * M[3][2]) -
			M[1][1] * (M[0][2] * M[3][3] - M[0][3] * M[3][2]) +
			M[3][1] * (M[0][2] * M[1][3] - M[0][3] * M[1][2])
			) -
		M[3][0] * (
			M[0][1] * (M[1][2] * M[2][3] - M[1][3] * M[2][2]) -
			M[1][1] * (M[0][2] * M[2][3] - M[0][3] * M[2][2]) +
			M[2][1] * (M[0][2] * M[1][3] - M[0][3] * M[1][2])
			);
}

/** Calculate determinant of rotation 3x3 matrix */
template<typename T>
inline T TMatrix<T>::RotDeterminant() const
{
	return
		M[0][0] * (M[1][1] * M[2][2] - M[1][2] * M[2][1]) -
		M[1][0] * (M[0][1] * M[2][2] - M[0][2] * M[2][1]) +
		M[2][0] * (M[0][1] * M[1][2] - M[0][2] * M[1][1]);
}

// InverseFast.
// Unlike Inverse, this may ensure if used on nil/nan matrix in non-final builds
//   should not be used on matrices that may be nil/nan
// note: not actually faster than Inverse
template<typename T>
inline TMatrix<T> TMatrix<T>::InverseFast() const
{
	// If we're in non final release, then make sure we're not creating NaNs
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Check for zero scale matrix to invert
	// @todo : remove this axis length check
	if (GetScaledAxis(EAxis::X).IsNearlyZero(UE_SMALL_NUMBER) &&
		GetScaledAxis(EAxis::Y).IsNearlyZero(UE_SMALL_NUMBER) &&
		GetScaledAxis(EAxis::Z).IsNearlyZero(UE_SMALL_NUMBER))
	{
		ErrorEnsure(TEXT("TMatrix<T>::InverseFast(), trying to invert a NIL matrix, this results in NaNs! Use Inverse() instead."));
	}
#endif

	TMatrix<T> Result;
	if ( ! VectorMatrixInverse(&Result, this) )
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		ErrorEnsure(TEXT("TMatrix<T>::InverseFast(), trying to invert a non-invertible matrix, this results in NaNs! Use Inverse() instead."));
#endif
	}
	Result.DiagnosticCheckNaN(); // <- pointless, VectorMatrixInverse ensures output is valid
	return Result;
}

// Inverse.   zero & NaN matrices are silently changed to identity
template<typename T>
inline TMatrix<T> TMatrix<T>::Inverse() const
{
	// Check for zero scale matrix to invert
	// @todo : remove this axis length check
	if (GetScaledAxis(EAxis::X).IsNearlyZero(UE_SMALL_NUMBER) &&
		GetScaledAxis(EAxis::Y).IsNearlyZero(UE_SMALL_NUMBER) &&
		GetScaledAxis(EAxis::Z).IsNearlyZero(UE_SMALL_NUMBER))
	{
		// just set to zero - avoids unsafe inverse of zero and duplicates what QNANs were resulting in before (scaling away all children)
		return Identity;
	}
	
	TMatrix<T> Result;
	// VectorMatrixInverse will return false and fill identity for non-invertible matrices
	VectorMatrixInverse(&Result, this);
	Result.DiagnosticCheckNaN(); // <- pointless, VectorMatrixInverse ensures output is valid
	return Result;
}

template<typename T>
inline TMatrix<T> TMatrix<T>::TransposeAdjoint() const
{
	TMatrix<T> TA;

	TA.M[0][0] = this->M[1][1] * this->M[2][2] - this->M[1][2] * this->M[2][1];
	TA.M[0][1] = this->M[1][2] * this->M[2][0] - this->M[1][0] * this->M[2][2];
	TA.M[0][2] = this->M[1][0] * this->M[2][1] - this->M[1][1] * this->M[2][0];
	TA.M[0][3] = 0.f;

	TA.M[1][0] = this->M[2][1] * this->M[0][2] - this->M[2][2] * this->M[0][1];
	TA.M[1][1] = this->M[2][2] * this->M[0][0] - this->M[2][0] * this->M[0][2];
	TA.M[1][2] = this->M[2][0] * this->M[0][1] - this->M[2][1] * this->M[0][0];
	TA.M[1][3] = 0.f;

	TA.M[2][0] = this->M[0][1] * this->M[1][2] - this->M[0][2] * this->M[1][1];
	TA.M[2][1] = this->M[0][2] * this->M[1][0] - this->M[0][0] * this->M[1][2];
	TA.M[2][2] = this->M[0][0] * this->M[1][1] - this->M[0][1] * this->M[1][0];
	TA.M[2][3] = 0.f;

	TA.M[3][0] = 0.f;
	TA.M[3][1] = 0.f;
	TA.M[3][2] = 0.f;
	TA.M[3][3] = 1.f;

	TA.DiagnosticCheckNaN();
	return TA;
}

// NOTE: Tolerance is no longer used and remains for API backwards compatibility reasons
template<typename T>
inline void TMatrix<T>::RemoveScaling(T /*Tolerance=UE_SMALL_NUMBER*/)
{
	// For each row, find magnitude, and if its non-zero re-scale so its unit length.
	const T SquareSum0 = (M[0][0] * M[0][0]) + (M[0][1] * M[0][1]) + (M[0][2] * M[0][2]);
	const T SquareSum1 = (M[1][0] * M[1][0]) + (M[1][1] * M[1][1]) + (M[1][2] * M[1][2]);
	const T SquareSum2 = (M[2][0] * M[2][0]) + (M[2][1] * M[2][1]) + (M[2][2] * M[2][2]);
	const T Scale0 = SquareSum0 > T(0) ? FMath::InvSqrt(SquareSum0) : T(1);
	const T Scale1 = SquareSum1 > T(0) ? FMath::InvSqrt(SquareSum1) : T(1);
	const T Scale2 = SquareSum2 > T(0) ? FMath::InvSqrt(SquareSum2) : T(1);
	M[0][0] *= Scale0;
	M[0][1] *= Scale0;
	M[0][2] *= Scale0;
	M[1][0] *= Scale1;
	M[1][1] *= Scale1;
	M[1][2] *= Scale1;
	M[2][0] *= Scale2;
	M[2][1] *= Scale2;
	M[2][2] *= Scale2;
	DiagnosticCheckNaN();
}

// Returns matrix without scale information
template<typename T>
inline TMatrix<T> TMatrix<T>::GetMatrixWithoutScale(T Tolerance/*=UE_SMALL_NUMBER*/) const
{
	TMatrix<T> Result = (TMatrix<T>&)*this;
	Result.RemoveScaling(Tolerance);
	return Result;
}

/** Remove any scaling from this matrix (ie magnitude of each row is 1) and return the 3D scale vector that was initially present. */
template<typename T>
inline TVector<T> TMatrix<T>::ExtractScaling(T Tolerance/*=UE_SMALL_NUMBER*/)
{
	TVector<T> Scale3D(0, 0, 0);

	// For each row, find magnitude, and if its non-zero re-scale so its unit length.
	const T SquareSum0 = (M[0][0] * M[0][0]) + (M[0][1] * M[0][1]) + (M[0][2] * M[0][2]);
	const T SquareSum1 = (M[1][0] * M[1][0]) + (M[1][1] * M[1][1]) + (M[1][2] * M[1][2]);
	const T SquareSum2 = (M[2][0] * M[2][0]) + (M[2][1] * M[2][1]) + (M[2][2] * M[2][2]);

	if (SquareSum0 > Tolerance)
	{
		T Scale0 = FMath::Sqrt(SquareSum0);
		Scale3D[0] = Scale0;
		T InvScale0 = 1.f / Scale0;
		M[0][0] *= InvScale0;
		M[0][1] *= InvScale0;
		M[0][2] *= InvScale0;
	}
	else
	{
		Scale3D[0] = 0;
	}

	if (SquareSum1 > Tolerance)
	{
		T Scale1 = FMath::Sqrt(SquareSum1);
		Scale3D[1] = Scale1;
		T InvScale1 = 1.f / Scale1;
		M[1][0] *= InvScale1;
		M[1][1] *= InvScale1;
		M[1][2] *= InvScale1;
	}
	else
	{
		Scale3D[1] = 0;
	}

	if (SquareSum2 > Tolerance)
	{
		T Scale2 = FMath::Sqrt(SquareSum2);
		Scale3D[2] = Scale2;
		T InvScale2 = 1.f / Scale2;
		M[2][0] *= InvScale2;
		M[2][1] *= InvScale2;
		M[2][2] *= InvScale2;
	}
	else
	{
		Scale3D[2] = 0;
	}

	return Scale3D;
}

/** return a 3D scale vector calculated from this matrix (where each component is the magnitude of a row vector). */
template<typename T>
inline TVector<T> TMatrix<T>::GetScaleVector(T Tolerance/*=UE_SMALL_NUMBER*/) const
{
	TVector<T> Scale3D(1, 1, 1);

	// For each row, find magnitude, and if its non-zero re-scale so its unit length.
	for (int32 i = 0; i < 3; i++)
	{
		const T SquareSum = (M[i][0] * M[i][0]) + (M[i][1] * M[i][1]) + (M[i][2] * M[i][2]);
		if (SquareSum > Tolerance)
		{
			Scale3D[i] = FMath::Sqrt(SquareSum);
		}
		else
		{
			Scale3D[i] = 0.f;
		}
	}

	return Scale3D;
}
// Remove any translation from this matrix
template<typename T>
inline TMatrix<T> TMatrix<T>::RemoveTranslation() const
{
	TMatrix<T> Result = (TMatrix<T>&)*this;
	Result.M[3][0] = 0.0f;
	Result.M[3][1] = 0.0f;
	Result.M[3][2] = 0.0f;
	return Result;
}

template<typename T>
inline TMatrix<T> TMatrix<T>::ConcatTranslation(const TVector<T>& Translation) const
{
	TMatrix<T> Result;

	T* RESTRICT Dest = &Result.M[0][0];
	const T* RESTRICT Src = &M[0][0];
	const T* RESTRICT Trans = &Translation.X;

	Dest[0] = Src[0];
	Dest[1] = Src[1];
	Dest[2] = Src[2];
	Dest[3] = Src[3];
	Dest[4] = Src[4];
	Dest[5] = Src[5];
	Dest[6] = Src[6];
	Dest[7] = Src[7];
	Dest[8] = Src[8];
	Dest[9] = Src[9];
	Dest[10] = Src[10];
	Dest[11] = Src[11];
	Dest[12] = Src[12] + Trans[0];
	Dest[13] = Src[13] + Trans[1];
	Dest[14] = Src[14] + Trans[2];
	Dest[15] = Src[15];

	DiagnosticCheckNaN();
	return Result;
}

/** Returns true if any element of this matrix is not finite */
template<typename T>
inline bool TMatrix<T>::ContainsNaN() const
{
	for (int32 i = 0; i < 4; i++)
	{
		for (int32 j = 0; j < 4; j++)
		{
			if (!FMath::IsFinite(M[i][j]))
			{
				return true;
			}
		}
	}

	return false;
}

/** @return the minimum magnitude of any row of the matrix. */
template<typename T>
inline T TMatrix<T>::GetMinimumAxisScale() const
{
	const T MaxRowScaleSquared = FMath::Min(
		GetScaledAxis(EAxis::X).SizeSquared(),
		FMath::Min(
			GetScaledAxis(EAxis::Y).SizeSquared(),
			GetScaledAxis(EAxis::Z).SizeSquared()
		)
	);
	return FMath::Sqrt(MaxRowScaleSquared);
}

/** @return the maximum magnitude of any row of the matrix. */
template<typename T>
inline T TMatrix<T>::GetMaximumAxisScale() const
{
	const T MaxRowScaleSquared = FMath::Max(
		GetScaledAxis(EAxis::X).SizeSquared(),
		FMath::Max(
			GetScaledAxis(EAxis::Y).SizeSquared(),
			GetScaledAxis(EAxis::Z).SizeSquared()
		)
	);
	return FMath::Sqrt(MaxRowScaleSquared);
}

template<typename T>
inline void TMatrix<T>::ScaleTranslation(const TVector<T>& InScale3D)
{
	M[3][0] *= InScale3D.X;
	M[3][1] *= InScale3D.Y;
	M[3][2] *= InScale3D.Z;
}

// GetOrigin

template<typename T>
inline TVector<T> TMatrix<T>::GetOrigin() const
{
	return TVector<T>(M[3][0], M[3][1], M[3][2]);
}

template<typename T>
inline TVector<T> TMatrix<T>::GetScaledAxis(EAxis::Type InAxis) const
{
	switch (InAxis)
	{
	case EAxis::X:
		return TVector<T>(M[0][0], M[0][1], M[0][2]);

	case EAxis::Y:
		return TVector<T>(M[1][0], M[1][1], M[1][2]);

	case EAxis::Z:
		return TVector<T>(M[2][0], M[2][1], M[2][2]);

	case EAxis::None:
	default:
		ensure(0);
		return TVector<T>::ZeroVector;
	}
}

template<typename T>
inline void TMatrix<T>::GetScaledAxes(TVector<T>& X, TVector<T>& Y, TVector<T>& Z) const
{
	X.X = M[0][0]; X.Y = M[0][1]; X.Z = M[0][2];
	Y.X = M[1][0]; Y.Y = M[1][1]; Y.Z = M[1][2];
	Z.X = M[2][0]; Z.Y = M[2][1]; Z.Z = M[2][2];
}

template<typename T>
inline TVector<T> TMatrix<T>::GetUnitAxis(EAxis::Type InAxis) const
{
	return GetScaledAxis(InAxis).GetSafeNormal();
}

template<typename T>
inline void TMatrix<T>::GetUnitAxes(TVector<T>& X, TVector<T>& Y, TVector<T>& Z) const
{
	GetScaledAxes(X, Y, Z);
	X.Normalize();
	Y.Normalize();
	Z.Normalize();
}

template<typename T>
inline void TMatrix<T>::SetAxis(int32 i, const TVector<T>& Axis)
{
	checkSlow(i >= 0 && i <= 2);
	M[i][0] = Axis.X;
	M[i][1] = Axis.Y;
	M[i][2] = Axis.Z;
	DiagnosticCheckNaN();
}

template<typename T>
inline void TMatrix<T>::SetOrigin(const TVector<T>& NewOrigin)
{
	M[3][0] = NewOrigin.X;
	M[3][1] = NewOrigin.Y;
	M[3][2] = NewOrigin.Z;
	DiagnosticCheckNaN();
}

template<typename T>
inline void TMatrix<T>::SetAxes(const TVector<T>* Axis0 /*= NULL*/, const TVector<T>* Axis1 /*= NULL*/, const TVector<T>* Axis2 /*= NULL*/, const TVector<T>* Origin /*= NULL*/)
{
	if (Axis0 != NULL)
	{
		M[0][0] = Axis0->X;
		M[0][1] = Axis0->Y;
		M[0][2] = Axis0->Z;
	}
	if (Axis1 != NULL)
	{
		M[1][0] = Axis1->X;
		M[1][1] = Axis1->Y;
		M[1][2] = Axis1->Z;
	}
	if (Axis2 != NULL)
	{
		M[2][0] = Axis2->X;
		M[2][1] = Axis2->Y;
		M[2][2] = Axis2->Z;
	}
	if (Origin != NULL)
	{
		M[3][0] = Origin->X;
		M[3][1] = Origin->Y;
		M[3][2] = Origin->Z;
	}
	DiagnosticCheckNaN();
}

template<typename T>
inline TVector<T> TMatrix<T>::GetColumn(int32 i) const
{
	checkSlow(i >= 0 && i <= 3);
	return TVector<T>(M[0][i], M[1][i], M[2][i]);
}

template<typename T>
inline void TMatrix<T>::SetColumn(int32 i, TVector<T> Value)
{
	checkSlow(i >= 0 && i <= 3);
	M[0][i] = Value.X;
	M[1][i] = Value.Y;
	M[2][i] = Value.Z;
}

template <typename T>
inline bool MakeFrustumPlane(T A, T B, T C, T D, TPlane<T>& OutPlane)
{
	const T	LengthSquared = A * A + B * B + C * C;
	if (LengthSquared > UE_DELTA * UE_DELTA)
	{
		const T	InvLength = FMath::InvSqrt(LengthSquared);
		OutPlane = TPlane<T>(-A * InvLength, -B * InvLength, -C * InvLength, D * InvLength);
		return 1;
	}
	else
		return 0;
}

// Frustum plane extraction. Assumes reverse Z. Near is depth == 1. Far is depth == 0.
template<typename T>
inline bool TMatrix<T>::GetFrustumNearPlane(TPlane<T>& OutPlane) const
{
	return MakeFrustumPlane(
		M[0][3] - M[0][2],
		M[1][3] - M[1][2],
		M[2][3] - M[2][2],
		M[3][3] - M[3][2],
		OutPlane
	);
}

template<typename T>
inline bool TMatrix<T>::GetFrustumFarPlane(TPlane<T>& OutPlane) const
{
	return MakeFrustumPlane(
		M[0][2],
		M[1][2],
		M[2][2],
		M[3][2],
		OutPlane
	);
}

template<typename T>
inline bool TMatrix<T>::GetFrustumLeftPlane(TPlane<T>& OutPlane) const
{
	return MakeFrustumPlane(
		M[0][3] + M[0][0],
		M[1][3] + M[1][0],
		M[2][3] + M[2][0],
		M[3][3] + M[3][0],
		OutPlane
	);
}

template<typename T>
inline bool TMatrix<T>::GetFrustumRightPlane(TPlane<T>& OutPlane) const
{
	return MakeFrustumPlane(
		M[0][3] - M[0][0],
		M[1][3] - M[1][0],
		M[2][3] - M[2][0],
		M[3][3] - M[3][0],
		OutPlane
	);
}

template<typename T>
inline bool TMatrix<T>::GetFrustumTopPlane(TPlane<T>& OutPlane) const
{
	return MakeFrustumPlane(
		M[0][3] - M[0][1],
		M[1][3] - M[1][1],
		M[2][3] - M[2][1],
		M[3][3] - M[3][1],
		OutPlane
	);
}

template<typename T>
inline bool TMatrix<T>::GetFrustumBottomPlane(TPlane<T>& OutPlane) const
{
	return MakeFrustumPlane(
		M[0][3] + M[0][1],
		M[1][3] + M[1][1],
		M[2][3] + M[2][1],
		M[3][3] + M[3][1],
		OutPlane
	);
}

/**
 * Utility for mirroring this transform across a certain plane,
 * and flipping one of the axis as well.
 */
template<typename T>
inline void TMatrix<T>::Mirror(EAxis::Type MirrorAxis, EAxis::Type FlipAxis)
{
	if (MirrorAxis == EAxis::X)
	{
		M[0][0] *= -1.f;
		M[1][0] *= -1.f;
		M[2][0] *= -1.f;

		M[3][0] *= -1.f;
	}
	else if (MirrorAxis == EAxis::Y)
	{
		M[0][1] *= -1.f;
		M[1][1] *= -1.f;
		M[2][1] *= -1.f;

		M[3][1] *= -1.f;
	}
	else if (MirrorAxis == EAxis::Z)
	{
		M[0][2] *= -1.f;
		M[1][2] *= -1.f;
		M[2][2] *= -1.f;

		M[3][2] *= -1.f;
	}

	if (FlipAxis == EAxis::X)
	{
		M[0][0] *= -1.f;
		M[0][1] *= -1.f;
		M[0][2] *= -1.f;
	}
	else if (FlipAxis == EAxis::Y)
	{
		M[1][0] *= -1.f;
		M[1][1] *= -1.f;
		M[1][2] *= -1.f;
	}
	else if (FlipAxis == EAxis::Z)
	{
		M[2][0] *= -1.f;
		M[2][1] *= -1.f;
		M[2][2] *= -1.f;
	}
}

/**
 * Apply Scale to this matrix
 */
template<typename T>
inline TMatrix<T> TMatrix<T>::ApplyScale(T Scale) const
{
	TMatrix<T> ScaleMatrix(
		TPlane<T>(Scale, 0.0f, 0.0f, 0.0f),
		TPlane<T>(0.0f, Scale, 0.0f, 0.0f),
		TPlane<T>(0.0f, 0.0f, Scale, 0.0f),
		TPlane<T>(0.0f, 0.0f, 0.0f, 1.0f)
	);
	return ScaleMatrix * ((TMatrix<T>&)*this);
}


/**
 * TPlane inline functions.
 */

template<typename T>
inline TPlane<T> TPlane<T>::TransformBy(const TMatrix<T>& M) const
{
	const TMatrix<T> tmpTA = M.TransposeAdjoint();
	const T DetM = M.Determinant();
	return this->TransformByUsingAdjointT(M, DetM, tmpTA);
}

template<typename T>
inline TPlane<T> UE::Math::TPlane<T>::TransformByUsingAdjointT(const TMatrix<T>& M, T DetM, const TMatrix<T>& TA) const
{
	TVector<T> newNorm = TA.TransformVector(*this).GetSafeNormal();

	if (DetM < 0.f)
	{
		newNorm *= -1.0f;
	}

	return TPlane<T>(M.TransformPosition(*this * W), newNorm);
}

/**
* TMatrix variation inline functions.
*/

template<typename T>
inline TBasisVectorMatrix<T>::TBasisVectorMatrix(const TVector<T>& XAxis,const TVector<T>& YAxis,const TVector<T>& ZAxis,const TVector<T>& Origin)
{
	for(uint32 RowIndex = 0;RowIndex < 3;RowIndex++)
	{
		M[RowIndex][0] = (&XAxis.X)[RowIndex];
		M[RowIndex][1] = (&YAxis.X)[RowIndex];
		M[RowIndex][2] = (&ZAxis.X)[RowIndex];
		M[RowIndex][3] = 0.0f;
	}
	M[3][0] = Origin | XAxis;
	M[3][1] = Origin | YAxis;
	M[3][2] = Origin | ZAxis;
	M[3][3] = 1.0f;
	this->DiagnosticCheckNaN();
}


template<typename T>
inline TLookFromMatrix<T>::TLookFromMatrix(const TVector<T>& EyePosition, const TVector<T>& LookDirection, const TVector<T>& UpVector)
{
	const TVector<T> ZAxis = LookDirection.GetSafeNormal();
	const TVector<T> XAxis = (UpVector ^ ZAxis).GetSafeNormal();
	const TVector<T> YAxis = ZAxis ^ XAxis;

	for (uint32 RowIndex = 0; RowIndex < 3; RowIndex++)
	{
		M[RowIndex][0] = (&XAxis.X)[RowIndex];
		M[RowIndex][1] = (&YAxis.X)[RowIndex];
		M[RowIndex][2] = (&ZAxis.X)[RowIndex];
		M[RowIndex][3] = 0.0f;
	}
	M[3][0] = -EyePosition | XAxis;
	M[3][1] = -EyePosition | YAxis;
	M[3][2] = -EyePosition | ZAxis;
	M[3][3] = 1.0f;
	this->DiagnosticCheckNaN();
}


template<typename T>
UE_FORCEINLINE_HINT TLookAtMatrix<T>::TLookAtMatrix(const TVector<T>& EyePosition, const TVector<T>& LookAtPosition, const TVector<T>& UpVector) :
	TLookFromMatrix<T>(EyePosition, LookAtPosition - EyePosition, UpVector)
{
}


} // namespace UE::Core
} // namespace UE
