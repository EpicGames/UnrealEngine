// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __MATRIX_ISPH__
#define __MATRIX_ISPH__

#include "Math/Vector.isph"

#ifndef EXPLICIT_MATRIX
#define EXPLICIT_MATRIX 0
#endif

struct FMatrix44f
{
#if EXPLICIT_MATRIX == 0
	float M[16];
#else
	float<16> M;
#endif
};

struct FMatrix
{
	double M[16];
};

typedef FMatrix FMatrix44d;

#ifndef EXPLICIT_MATRIX3x4
#define EXPLICIT_MATRIX3x4 0
#endif

struct FMatrix3x4
{
#if EXPLICIT_MATRIX3x4 == 0
	float M[12];
#else
	float<12> M;
#endif
};

static const uniform struct FMatrix44f FloatMatrixIdentity = {{ 1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1}};
static const uniform struct FMatrix44d DoubleMatrixIdentity = {{ 1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1}};

#define MatrixIdentity DoubleMatrixIdentity

inline uniform FMatrix44d SetMatrix(const uniform FVector4d &R0, const uniform FVector4d &R1, const uniform FVector4d &R2, const uniform FVector4d &R3)
{
	const uniform FMatrix44d Result = {{ R0.V[0], R0.V[1], R0.V[2], R0.V[3],
									R1.V[0], R1.V[1], R1.V[2], R1.V[3],
									R2.V[0], R2.V[1], R2.V[2], R2.V[3],
									R3.V[0], R3.V[1], R3.V[2], R3.V[3] }};
	return Result;
}

inline uniform FMatrix44f SetMatrix(const uniform FVector4f &R0, const uniform FVector4f &R1, const uniform FVector4f &R2, const uniform FVector4f &R3)
{
	const uniform FMatrix44f Result = {{ R0.V[0], R0.V[1], R0.V[2], R0.V[3],
									R1.V[0], R1.V[1], R1.V[2], R1.V[3],
									R2.V[0], R2.V[1], R2.V[2], R2.V[3],
									R3.V[0], R3.V[1], R3.V[2], R3.V[3] }};
	return Result;
}

inline FMatrix SetMatrix(const FVector4 &R0, const FVector4 &R1, const FVector4 &R2, const FVector4 &R3)
{
	const FMatrix Result = {{ R0.V[0], R0.V[1], R0.V[2], R0.V[3],
							R1.V[0], R1.V[1], R1.V[2], R1.V[3],
							R2.V[0], R2.V[1], R2.V[2], R2.V[3],
							R3.V[0], R3.V[1], R3.V[2], R3.V[3] }};
	return Result;
}

inline FMatrix44d SetMatrix(const FVector4d &R0, const FVector4d &R1, const FVector4d &R2, const uniform FVector4d &R3)
{
	const FMatrix44d Result = {{ R0.V[0], R0.V[1], R0.V[2], R0.V[3],
							R1.V[0], R1.V[1], R1.V[2], R1.V[3],
							R2.V[0], R2.V[1], R2.V[2], R2.V[3],
							R3.V[0], R3.V[1], R3.V[2], R3.V[3] }};
	return Result;
}

inline FMatrix44f SetMatrix(const FVector4f &R0, const FVector4f &R1, const FVector4f &R2, const uniform FVector4f &R3)
{
	const FMatrix44f Result = {{ R0.V[0], R0.V[1], R0.V[2], R0.V[3],
							R1.V[0], R1.V[1], R1.V[2], R1.V[3],
							R2.V[0], R2.V[1], R2.V[2], R2.V[3],
							R3.V[0], R3.V[1], R3.V[2], R3.V[3] }};
	return Result;
}

inline uniform FMatrix operator*(const uniform FMatrix &Matrix1, const uniform FMatrix &Matrix2)
{
	uniform FVector4 R[4];

	const uniform FVector4 *uniform A	= (const uniform FVector4 *uniform) &Matrix1;
	const uniform FVector4 *uniform B	= (const uniform FVector4 *uniform) &Matrix2;

	for (uniform unsigned int i = 0; i < 4; i++)
	{
		R[i] = VectorReplicate( A[i], 0 ) * B[0] +
				VectorReplicate( A[i], 2 ) * B[2] +
				VectorReplicate( A[i], 1 ) * B[1] +
				VectorReplicate( A[i], 3 ) * B[3];
	}

	return SetMatrix(R[0], R[1], R[2], R[3]);
}

inline FMatrix operator*(const FMatrix &A, const FMatrix &B)
{
	FMatrix Result;

	for (uniform unsigned int m = 0; m < 4; m++) 
	{
		varying FReal Sum;
		for (uniform unsigned int k = 0; k < 4; k++) 
		{
			Sum = 0;
			for (uniform unsigned int n = 0; n < 4; n++) 
			{
				Sum += A.M[m * 4 + n] * B.M[n * 4 + k];
			}
			
			Result.M[m * 4 + k] = Sum;
		}
	}

	return Result;
}

inline FMatrix operator*(const FMatrix &A, const uniform FMatrix &B)
{
	FMatrix Result;

	for (uniform unsigned int m = 0; m < 4; m++) 
	{
		varying FReal Sum;
		for (uniform unsigned int k = 0; k < 4; k++) 
		{
			Sum = 0;
			for (uniform unsigned int n = 0; n < 4; n++) 
			{
				Sum += A.M[m * 4 + n] * B.M[n * 4 + k];
			}
			
			Result.M[m * 4 + k] = Sum;
		}
	}

	return Result;
}

// Remove any scaling from this matrix (ie magnitude of each row is 1)
inline void MatrixRemoveScaling(uniform FMatrix &M)
{
	// For each row, find magnitude, and if its non-zero re-scale so its unit length.
	const uniform FReal SquareSum0 = (M.M[0] * M.M[0]) + (M.M[1] * M.M[1]) + (M.M[2] * M.M[2]);
	const uniform FReal SquareSum1 = (M.M[4] * M.M[4]) + (M.M[5] * M.M[5]) + (M.M[6] * M.M[6]);
	const uniform FReal SquareSum2 = (M.M[8] * M.M[8]) + (M.M[9] * M.M[9]) + (M.M[10] * M.M[10]);
	const uniform FReal Scale0 = select( SquareSum0 - SMALL_NUMBER >= ZERO, InvSqrt(SquareSum0), ONE );
	const uniform FReal Scale1 = select( SquareSum1 - SMALL_NUMBER >= ZERO, InvSqrt(SquareSum1), ONE );
	const uniform FReal Scale2 = select( SquareSum2 - SMALL_NUMBER >= ZERO, InvSqrt(SquareSum2), ONE );
	M.M[0] *= Scale0;
	M.M[1] *= Scale0;
	M.M[2] *= Scale0;
	M.M[4] *= Scale1;
	M.M[5] *= Scale1;
	M.M[6] *= Scale1;
	M.M[8] *= Scale2;
	M.M[9] *= Scale2;
	M.M[10] *= Scale2;
}

inline uniform FVector MatrixGetOrigin(const uniform FMatrix &M)
{
	return SetVector(M.M[12], M.M[13], M.M[14]);
}

template <typename T, typename V>
inline void MatrixGetScaledAxes(const T& M, V &X, V &Y, V &Z)
{
	X = SetVector(M.M[0], M.M[1], M.M[2]);
	Y = SetVector(M.M[4], M.M[5], M.M[6]);
	Z = SetVector(M.M[8], M.M[9], M.M[10]);
}

inline void MatrixScaleAxis(uniform FMatrix &M, const uniform int Axis, const uniform FReal ScalingFactor)
{
	varying FReal S0;
	*((uniform FVector *uniform)&S0) = *((uniform FVector *uniform)&M.M[(4*Axis)]);

#if EXPLICIT_VECTOR == 1
	// Using explicit vector aligns to 16 bytes. Save the 4th component when pulling from Matrix type
	const uniform FReal Saved = extract(S0, 3);
	S0 = insert(S0, 3, 0);
#endif

	S0 = S0 * ScalingFactor;

#if EXPLICIT_VECTOR == 1
	// Reintroduce 4th component here
	S0 = insert(S0, 3, Saved);
#endif

	*((uniform FVector *uniform)&M.M[(4*Axis)]) = *((uniform FVector *uniform)&S0);
}

inline uniform bool IsAnyMatrixScaledAxesNearlyZero(const uniform FMatrix &M, const uniform FReal Tolerance)
{
	uniform FVector X, Y, Z;
	MatrixGetScaledAxes(M, X, Y, Z);

	if(VectorIsAllNearlyZero(X, Tolerance) || VectorIsAllNearlyZero(Y, Tolerance) || VectorIsAllNearlyZero(Z, Tolerance))
	{
		return true;
	}

	return false;
}

inline uniform bool IsAllMatrixScaledAxesNearlyZero(const uniform FMatrix44d &M, const uniform double Tolerance)
{
	uniform FVector3d X, Y, Z;
	MatrixGetScaledAxes(M, X, Y, Z);

	if(VectorIsAllNearlyZero(X, Tolerance) && VectorIsAllNearlyZero(Y, Tolerance) && VectorIsAllNearlyZero(Z, Tolerance))
	{
		return true;
	}

	return false;
}

inline uniform bool IsAllMatrixScaledAxesNearlyZero(const uniform FMatrix44f &M, const uniform float Tolerance)
{
	uniform FVector3f X, Y, Z;
	MatrixGetScaledAxes(M, X, Y, Z);

	if(VectorIsAllNearlyZero(X, Tolerance) && VectorIsAllNearlyZero(Y, Tolerance) && VectorIsAllNearlyZero(Z, Tolerance))
	{
		return true;
	}

	return false;
}

inline uniform FMatrix MatrixTranspose(const uniform FMatrix& M)
{
	uniform FMatrix Result;

	const uniform FVector4 R1 = *((uniform FVector4 *uniform)&M.M[0]);
	const uniform FVector4 R2 = *((uniform FVector4 *uniform)&M.M[4]);
	const uniform FVector4 R3 = *((uniform FVector4 *uniform)&M.M[8]);
	const uniform FVector4 R4 = *((uniform FVector4 *uniform)&M.M[12]);

	*((uniform FVector4 *uniform)&Result.M[0]) = SetVector4(R1.V[0], R2.V[0], R3.V[0], R4.V[0]);
	*((uniform FVector4 *uniform)&Result.M[4]) = SetVector4(R1.V[1], R2.V[1], R3.V[1], R4.V[1]);
	*((uniform FVector4 *uniform)&Result.M[8]) = SetVector4(R1.V[2], R2.V[2], R3.V[2], R4.V[2]);
	*((uniform FVector4 *uniform)&Result.M[12]) = SetVector4(R1.V[3], R2.V[3], R3.V[3], R4.V[3]);

	return Result;
}

// for row major matrix
// we use __m128 to represent 2x2 matrix as A = | A0  A1 |
//                                              | A2  A3 |
// 2x2 row major Matrix multiply A*B
template <typename T>
static inline uniform T Mat2Mul(const uniform T& vec1, const uniform T& vec2)
{
	return
		VectorAdd(VectorMultiply( vec1, VectorSwizzle(vec2, 0,3,0,3)),
			VectorMultiply(VectorSwizzle(vec1, 1,0,3,2), VectorSwizzle(vec2, 2,1,2,1)));
}

// 2x2 row major Matrix adjugate multiply (A#)*B
template <typename T>
static inline uniform T Mat2AdjMul(const uniform T& vec1, const uniform T& vec2)
{
	return
		VectorSubtract(VectorMultiply(VectorSwizzle(vec1, 3,3,0,0), vec2),
			VectorMultiply(VectorSwizzle(vec1, 1,1,2,2), VectorSwizzle(vec2, 2,3,0,1)));

}

// 2x2 row major Matrix multiply adjugate A*(B#)
template <typename T>
static inline uniform T Mat2MulAdj(const uniform T& vec1, const uniform T& vec2)
{
	return
		VectorSubtract(VectorMultiply( vec1, VectorSwizzle(vec2, 3,0,3,0)),
			VectorMultiply(VectorSwizzle(vec1, 1,0,3,2), VectorSwizzle(vec2, 2,1,2,1)));
}

inline uniform double MatrixDeterminant(const uniform FMatrix44d& M)
{
	const uniform FVector4d *uniform MVec = (const uniform FVector4d *uniform) &M;

	// sub matrices
	const uniform FVector4d A = VectorShuffle(MVec[0], MVec[1], 0,1,0,1);
	const uniform FVector4d B = VectorShuffle(MVec[0], MVec[1], 2,3,2,3);
	const uniform FVector4d C = VectorShuffle(MVec[2], MVec[3], 0,1,0,1);
	const uniform FVector4d D = VectorShuffle(MVec[2], MVec[3], 2,3,2,3);

	// determinant as (|A| |B| |C| |D|)
	const uniform FVector4d detSub = VectorSubtract(
		VectorMultiply(VectorShuffle(MVec[0], MVec[2], 0,2,0,2), VectorShuffle(MVec[1], MVec[3], 1,3,1,3)),
		VectorMultiply(VectorShuffle(MVec[0], MVec[2], 1,3,1,3), VectorShuffle(MVec[1], MVec[3], 0,2,0,2))
	);

	// D#C
	const uniform FVector4d D_C = Mat2AdjMul(D, C);
	// A#B
	const uniform FVector4d A_B = Mat2AdjMul(A, B);

	// |M| = |A|*|D| + |B|*|C|
	uniform double detM = (detSub.V[0] * detSub.V[3]) + (detSub.V[1] * detSub.V[2]);

	// tr((A#B)(D#C))
	uniform FVector4d tr = VectorMultiply(A_B, VectorSwizzle(D_C, 0,2,1,3));
	tr = tr + VectorSwizzle(tr, 2,3,0,1);
	tr = tr + VectorSwizzle(tr, 1,2,3,0);
	// |M| = |A|*|D| + |B|*|C| - tr((A#B)(D#C)
	detM = detM - tr.V[0];

	return detM;
}

inline uniform float MatrixDeterminant(const uniform FMatrix44f& M)
{
	const uniform FVector4f *uniform MVec = (const uniform FVector4f *uniform) &M;

	// sub matrices
	const uniform FVector4f A = VectorShuffle(MVec[0], MVec[1], 0,1,0,1);
	const uniform FVector4f B = VectorShuffle(MVec[0], MVec[1], 2,3,2,3);
	const uniform FVector4f C = VectorShuffle(MVec[2], MVec[3], 0,1,0,1);
	const uniform FVector4f D = VectorShuffle(MVec[2], MVec[3], 2,3,2,3);

	// determinant as (|A| |B| |C| |D|)
	const uniform FVector4f detSub = VectorSubtract(
		VectorMultiply(VectorShuffle(MVec[0], MVec[2], 0,2,0,2), VectorShuffle(MVec[1], MVec[3], 1,3,1,3)),
		VectorMultiply(VectorShuffle(MVec[0], MVec[2], 1,3,1,3), VectorShuffle(MVec[1], MVec[3], 0,2,0,2))
	);

	// D#C
	const uniform FVector4f D_C = Mat2AdjMul(D, C);
	// A#B
	const uniform FVector4f A_B = Mat2AdjMul(A, B);

	// |M| = |A|*|D| + |B|*|C|
	uniform float detM = (detSub.V[0] * detSub.V[3]) + (detSub.V[1] * detSub.V[2]);

	// tr((A#B)(D#C))
	uniform FVector4f tr = VectorMultiply(A_B, VectorSwizzle(D_C, 0,2,1,3));
	tr = tr + VectorSwizzle(tr, 2,3,0,1);
	tr = tr + VectorSwizzle(tr, 1,2,3,0);
	// |M| = |A|*|D| + |B|*|C| - tr((A#B)(D#C)
	detM = detM - tr.V[0];

	return detM;
}

inline uniform FMatrix44d VectorMatrixInverse(const uniform FMatrix44d& M)
{
	// use block matrix method
	// A is a matrix, then i(A) or iA means inverse of A, A# (or A_ in code) means adjugate of A, |A| (or detA in code) is determinant, tr(A) is trace
	const uniform FVector4d *uniform MVec = (const uniform FVector4d *uniform) &M;

	// sub matrices
	const uniform FVector4d A = VectorShuffle(MVec[0], MVec[1], 0,1,0,1);
	const uniform FVector4d B = VectorShuffle(MVec[0], MVec[1], 2,3,2,3);
	const uniform FVector4d C = VectorShuffle(MVec[2], MVec[3], 0,1,0,1);
	const uniform FVector4d D = VectorShuffle(MVec[2], MVec[3], 2,3,2,3);

	// determinant as (|A| |B| |C| |D|)
	const uniform FVector4d detSub = VectorSubtract(
		VectorMultiply(VectorShuffle(MVec[0], MVec[2], 0,2,0,2), VectorShuffle(MVec[1], MVec[3], 1,3,1,3)),
		VectorMultiply(VectorShuffle(MVec[0], MVec[2], 1,3,1,3), VectorShuffle(MVec[1], MVec[3], 0,2,0,2))
	);

	const uniform FVector4d detA = VectorSwizzle(detSub, 0,0,0,0);
	const uniform FVector4d detB = VectorSwizzle(detSub, 1,1,1,1);
	const uniform FVector4d detC = VectorSwizzle(detSub, 2,2,2,2);
	const uniform FVector4d detD = VectorSwizzle(detSub, 3,3,3,3);

	// let iM = 1/|M| * | X  Y |
	//                  | Z  W |

	// D#C
	const uniform FVector4d D_C = Mat2AdjMul(D, C);
	// A#B
	const uniform FVector4d A_B = Mat2AdjMul(A, B);
	// X# = |D|A - B(D#C)
	uniform FVector4d X_ = VectorSubtract(VectorMultiply(detD, A), Mat2Mul(B, D_C));
	// W# = |A|D - C(A#B)
	uniform FVector4d W_ = VectorSubtract(VectorMultiply(detA, D), Mat2Mul(C, A_B));

	// |M| = |A|*|D| + ... (continue later)
	uniform FVector4d detM = VectorMultiply(detA, detD);

	// Y# = |B|C - D(A#B)#
	uniform FVector4d Y_ = VectorSubtract(VectorMultiply(detB, C), Mat2MulAdj(D, A_B));
	// Z# = |C|B - A(D#C)#
	uniform FVector4d Z_ = VectorSubtract(VectorMultiply(detC, B), Mat2MulAdj(A, D_C));

	// |M| = |A|*|D| + |B|*|C| ... (continue later)
	detM = VectorAdd(detM, VectorMultiply(detB, detC));

	// tr((A#B)(D#C))
	uniform FVector4d tr = VectorMultiply(A_B, VectorSwizzle(D_C, 0,2,1,3));
	tr = tr + VectorSwizzle(tr, 2,3,0,1);
	tr = tr + VectorSwizzle(tr, 1,2,3,0);
	// |M| = |A|*|D| + |B|*|C| - tr((A#B)(D#C)
	detM = VectorSubtract(detM, tr);

	uniform FMatrix44d R = DoubleMatrixIdentity;

	if(detM.V[0] != DOUBLE_ZERO)
	{
		const uniform FVector4d adjSignMask = SetVector4(DOUBLE_ONE, -DOUBLE_ONE, -DOUBLE_ONE, DOUBLE_ONE);
		// (1/|M|, -1/|M|, -1/|M|, 1/|M|)
		const uniform FVector4d rDetM = VectorDivide(adjSignMask, detM);

		X_ = VectorMultiply(X_, rDetM);
		Y_ = VectorMultiply(Y_, rDetM);
		Z_ = VectorMultiply(Z_, rDetM);
		W_ = VectorMultiply(W_, rDetM);

		uniform FVector4d *uniform RVec = (uniform FVector4d *uniform) &R;

		// apply adjugate and store, here we combine adjugate shuffle and store shuffle
		RVec[0] = VectorShuffle(X_, Y_, 3,1,3,1);
		RVec[1] = VectorShuffle(X_, Y_, 2,0,2,0);
		RVec[2] = VectorShuffle(Z_, W_, 3,1,3,1);
		RVec[3] = VectorShuffle(Z_, W_, 2,0,2,0);	
	}

	return R;
}

inline uniform FMatrix44f VectorMatrixInverse(const uniform FMatrix44f& M)
{
	// use block matrix method
	// A is a matrix, then i(A) or iA means inverse of A, A# (or A_ in code) means adjugate of A, |A| (or detA in code) is determinant, tr(A) is trace
	const uniform FVector4f *uniform MVec = (const uniform FVector4f *uniform) &M;

	// sub matrices
	const uniform FVector4f A = VectorShuffle(MVec[0], MVec[1], 0,1,0,1);
	const uniform FVector4f B = VectorShuffle(MVec[0], MVec[1], 2,3,2,3);
	const uniform FVector4f C = VectorShuffle(MVec[2], MVec[3], 0,1,0,1);
	const uniform FVector4f D = VectorShuffle(MVec[2], MVec[3], 2,3,2,3);

	// determinant as (|A| |B| |C| |D|)
	const uniform FVector4f detSub = VectorSubtract(
		VectorMultiply(VectorShuffle(MVec[0], MVec[2], 0,2,0,2), VectorShuffle(MVec[1], MVec[3], 1,3,1,3)),
		VectorMultiply(VectorShuffle(MVec[0], MVec[2], 1,3,1,3), VectorShuffle(MVec[1], MVec[3], 0,2,0,2))
	);

	const uniform FVector4f detA = VectorSwizzle(detSub, 0,0,0,0);
	const uniform FVector4f detB = VectorSwizzle(detSub, 1,1,1,1);
	const uniform FVector4f detC = VectorSwizzle(detSub, 2,2,2,2);
	const uniform FVector4f detD = VectorSwizzle(detSub, 3,3,3,3);

	// let iM = 1/|M| * | X  Y |
	//                  | Z  W |

	// D#C
	const uniform FVector4f D_C = Mat2AdjMul(D, C);
	// A#B
	const uniform FVector4f A_B = Mat2AdjMul(A, B);
	// X# = |D|A - B(D#C)
	uniform FVector4f X_ = VectorSubtract(VectorMultiply(detD, A), Mat2Mul(B, D_C));
	// W# = |A|D - C(A#B)
	uniform FVector4f W_ = VectorSubtract(VectorMultiply(detA, D), Mat2Mul(C, A_B));

	// |M| = |A|*|D| + ... (continue later)
	uniform FVector4f detM = VectorMultiply(detA, detD);

	// Y# = |B|C - D(A#B)#
	uniform FVector4f Y_ = VectorSubtract(VectorMultiply(detB, C), Mat2MulAdj(D, A_B));
	// Z# = |C|B - A(D#C)#
	uniform FVector4f Z_ = VectorSubtract(VectorMultiply(detC, B), Mat2MulAdj(A, D_C));

	// |M| = |A|*|D| + |B|*|C| ... (continue later)
	detM = VectorAdd(detM, VectorMultiply(detB, detC));

	// tr((A#B)(D#C))
	uniform FVector4f tr = VectorMultiply(A_B, VectorSwizzle(D_C, 0,2,1,3));
	tr = tr + VectorSwizzle(tr, 2,3,0,1);
	tr = tr + VectorSwizzle(tr, 1,2,3,0);
	// |M| = |A|*|D| + |B|*|C| - tr((A#B)(D#C)
	detM = VectorSubtract(detM, tr);

	uniform FMatrix44f R = FloatMatrixIdentity;

	if(detM.V[0] != ZERO)
	{
		const uniform FVector4f adjSignMask = SetVector4(FLOAT_ONE, -FLOAT_ONE, -FLOAT_ONE, FLOAT_ONE);
		// (1/|M|, -1/|M|, -1/|M|, 1/|M|)
		const uniform FVector4f rDetM = VectorDivide(adjSignMask, detM);

		X_ = VectorMultiply(X_, rDetM);
		Y_ = VectorMultiply(Y_, rDetM);
		Z_ = VectorMultiply(Z_, rDetM);
		W_ = VectorMultiply(W_, rDetM);

		uniform FVector4f *uniform RVec = (uniform FVector4f *uniform) &R;

		// apply adjugate and store, here we combine adjugate shuffle and store shuffle
		RVec[0] = VectorShuffle(X_, Y_, 3,1,3,1);
		RVec[1] = VectorShuffle(X_, Y_, 2,0,2,0);
		RVec[2] = VectorShuffle(Z_, W_, 3,1,3,1);
		RVec[3] = VectorShuffle(Z_, W_, 2,0,2,0);	
	}

	return R;
}

inline uniform FMatrix44d MatrixInverse(const uniform FMatrix44d& M)
{
	uniform FMatrix44d Result;

	// Check for zero scale matrix to invert
	if(	IsAllMatrixScaledAxesNearlyZero(M, DOUBLE_SMALL_NUMBER) ) 
	{
		// just set to zero - avoids unsafe inverse of zero and duplicates what QNANs were resulting in before (scaling away all children)
		Result = DoubleMatrixIdentity;
	}
	else
	{
		Result = VectorMatrixInverse(M);
	}

	return Result;
}

inline uniform FMatrix44f MatrixInverse(const uniform FMatrix44f& M)
{
	uniform FMatrix44f Result;

	// Check for zero scale matrix to invert
	if(	IsAllMatrixScaledAxesNearlyZero(M, FLOAT_SMALL_NUMBER) ) 
	{
		// just set to zero - avoids unsafe inverse of zero and duplicates what QNANs were resulting in before (scaling away all children)
		Result = FloatMatrixIdentity;
	}
	else
	{
		Result = VectorMatrixInverse(M);
	}

	return Result;
}

template <typename T, typename V>
inline T VectorTransformVector(const T& VecP, const V& M)
{
	T VTempX, VTempY, VTempZ, VTempW;

	// Splat x,y,z and w
	VTempX = VectorReplicate(VecP, 0);
	VTempY = VectorReplicate(VecP, 1);
	VTempZ = VectorReplicate(VecP, 2);
	VTempW = VectorReplicate(VecP, 3);

	// Mul by the matrix
	VTempX = VectorMultiply(VTempX, SetVector4(M.M[0], M.M[1], M.M[2], M.M[3]));
	VTempY = VectorMultiply(VTempY, SetVector4(M.M[4], M.M[5], M.M[6], M.M[7]));
	VTempZ = VectorMultiply(VTempZ, SetVector4(M.M[8], M.M[9], M.M[10], M.M[11]));
	VTempW = VectorMultiply(VTempW, SetVector4(M.M[12], M.M[13], M.M[14], M.M[15]));

	// Add them all together
	VTempX = VectorAdd(VTempX, VTempY);
	VTempZ = VectorAdd(VTempZ, VTempW);
	VTempX = VectorAdd(VTempX, VTempZ);

	return VTempX;
}

// Calculate homogeneous transform. W component assumed to be 1.0
inline FVector MatrixTransformPosition(const FVector &P, const FMatrix &M)
{
	FVector VTempX, VTempY, VTempZ;

	// Splat x,y,z
	VTempX = SetVector(P.V[0], P.V[0], P.V[0]);
	VTempY = SetVector(P.V[1], P.V[1], P.V[1]);
	VTempZ = SetVector(P.V[2], P.V[2], P.V[2]);

	// Mul by the matrix
	VTempX = VTempX * SetVector(M.M[0], M.M[1], M.M[2]);
	VTempY = VTempY * SetVector(M.M[4], M.M[5], M.M[6]);
	VTempZ = VTempZ * SetVector(M.M[8], M.M[9], M.M[10]);
	const FVector VTempW = SetVector(M.M[12], M.M[13], M.M[14]);

	// Add them all together
	VTempX = VTempX + VTempY;
	VTempZ = VTempZ + VTempW;
	VTempX = VTempX + VTempZ;

	return VTempX;
}

// Calculate homogeneous transform. W component assumed to be 1.0
inline FVector3d MatrixTransformPosition(const FVector3d &P, const uniform FMatrix44d &M)
{
	FVector3d VTempX, VTempY, VTempZ;

	// Splat x,y,z
	VTempX = SetVector(P.V[0], P.V[0], P.V[0]);
	VTempY = SetVector(P.V[1], P.V[1], P.V[1]);
	VTempZ = SetVector(P.V[2], P.V[2], P.V[2]);

	// Mul by the matrix
	VTempX = VTempX * SetVector(M.M[0], M.M[1], M.M[2]);
	VTempY = VTempY * SetVector(M.M[4], M.M[5], M.M[6]);
	VTempZ = VTempZ * SetVector(M.M[8], M.M[9], M.M[10]);
	const uniform FVector3d VTempW = SetVector(M.M[12], M.M[13], M.M[14]);

	// Add them all together
	VTempX = VTempX + VTempY;
	VTempZ = VTempZ + VTempW;
	VTempX = VTempX + VTempZ;

	return VTempX;
}

inline FVector3f MatrixTransformPosition(const FVector3f &P, const uniform FMatrix44f &M)
{
	FVector3f VTempX, VTempY, VTempZ;

	// Splat x,y,z
	VTempX = SetVector(P.V[0], P.V[0], P.V[0]);
	VTempY = SetVector(P.V[1], P.V[1], P.V[1]);
	VTempZ = SetVector(P.V[2], P.V[2], P.V[2]);

	// Mul by the matrix
	VTempX = VTempX * SetVector(M.M[0], M.M[1], M.M[2]);
	VTempY = VTempY * SetVector(M.M[4], M.M[5], M.M[6]);
	VTempZ = VTempZ * SetVector(M.M[8], M.M[9], M.M[10]);
	const uniform FVector3f VTempW = SetVector(M.M[12], M.M[13], M.M[14]);

	// Add them all together
	VTempX = VTempX + VTempY;
	VTempZ = VTempZ + VTempW;
	VTempX = VTempX + VTempZ;

	return VTempX;
}

// Calculate homogeneous transform. W component assumed to be 1.0
inline uniform FVector3f MatrixTransformPosition(const uniform FVector3f& P, const uniform FMatrix44f& M)
{
	uniform FVector3f VTempX, VTempY, VTempZ;

	// Splat x,y,z
	VTempX = SetVector(P.V[0], P.V[0], P.V[0]);
	VTempY = SetVector(P.V[1], P.V[1], P.V[1]);
	VTempZ = SetVector(P.V[2], P.V[2], P.V[2]);

	// Mul by the matrix
	VTempX = VTempX * SetVector(M.M[0], M.M[1], M.M[2]);
	VTempY = VTempY * SetVector(M.M[4], M.M[5], M.M[6]);
	VTempZ = VTempZ * SetVector(M.M[8], M.M[9], M.M[10]);
	const uniform FVector3f VTempW = SetVector(M.M[12], M.M[13], M.M[14]);

	// Add them all together
	VTempX = VTempX + VTempY;
	VTempZ = VTempZ + VTempW;
	VTempX = VTempX + VTempZ;

	return VTempX;
}

// Calculate homogeneous transform. W component assumed to be 0.0
inline FVector MatrixTransformVector(const FVector &P, const FMatrix &M)
{
	FVector VTempX, VTempY, VTempZ;

	// Splat x,y,z
	VTempX = SetVector(P.V[0], P.V[0], P.V[0]);
	VTempY = SetVector(P.V[1], P.V[1], P.V[1]);
	VTempZ = SetVector(P.V[2], P.V[2], P.V[2]);

	// Mul by the matrix
	VTempX = VTempX * SetVector(M.M[0], M.M[1], M.M[2]);
	VTempY = VTempY * SetVector(M.M[4], M.M[5], M.M[6]);
	VTempZ = VTempZ * SetVector(M.M[8], M.M[9], M.M[10]);

	// Add them all together
	return VTempX + VTempY + VTempZ;
}

inline uniform FVector3d MatrixInverseTransformVector(const uniform FMatrix44d &M, const uniform FVector3d &V)
{
	const uniform FMatrix44d InvSelf = VectorMatrixInverse(M); // FMatrix::InverseFast();
	return SetVector(VectorTransformVector(SetVector4(V, DOUBLE_ZERO), InvSelf));
}

inline uniform FVector3f MatrixInverseTransformVector(const uniform FMatrix44f &M, const uniform FVector3f &V)
{
	const uniform FMatrix44f InvSelf = VectorMatrixInverse(M); // FMatrix::InverseFast();
	return SetVector(VectorTransformVector(SetVector4(V, FLOAT_ZERO), InvSelf));
}

template <typename T>
inline uniform T MatrixReduceAdd(const varying T& M)
{
	return SetMatrix(
		SetVector4(reduce_add(M.M[0]), reduce_add(M.M[1]), reduce_add(M.M[2]), reduce_add(M.M[3])),
		SetVector4(reduce_add(M.M[4]), reduce_add(M.M[5]), reduce_add(M.M[6]), reduce_add(M.M[7])),
		SetVector4(reduce_add(M.M[8]), reduce_add(M.M[9]), reduce_add(M.M[10]), reduce_add(M.M[11])),
		SetVector4(reduce_add(M.M[12]), reduce_add(M.M[13]), reduce_add(M.M[14]), reduce_add(M.M[15]))
		);
}

#endif
