// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Plane.h"
#include "Math/Matrix.h"

namespace UE {
namespace Math {

 /**
 * Mirrors a point about an arbitrary plane 
 */
template<typename T>
struct TMirrorMatrix
	: public TMatrix<T>
{
public:

	/** 
	 * Constructor. Updated for the fact that our FPlane uses Ax+By+Cz=D.
	 * 
	 * @param Plane source plane for mirroring (assumed normalized)
	 */
	TMirrorMatrix( const TPlane<T>& Plane );

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TMirrorMatrix(const TMirrorMatrix<FArg>& From)
		: TMatrix<T>(From)
	{
	}
};

template<typename T>
inline TMirrorMatrix<T>::TMirrorMatrix( const TPlane<T>& Plane ) :
FMatrix(
	TPlane<T>( -2.f*Plane.X*Plane.X + 1.f,		-2.f*Plane.Y*Plane.X,		-2.f*Plane.Z*Plane.X,		0.f ),
	TPlane<T>( -2.f*Plane.X*Plane.Y,			-2.f*Plane.Y*Plane.Y + 1.f,	-2.f*Plane.Z*Plane.Y,		0.f ),
	TPlane<T>( -2.f*Plane.X*Plane.Z,			-2.f*Plane.Y*Plane.Z,		-2.f*Plane.Z*Plane.Z + 1.f,	0.f ),
	TPlane<T>(  2.f*Plane.X*Plane.W,			 2.f*Plane.Y*Plane.W,		 2.f*Plane.Z*Plane.W,		1.f ) )
{
	//check( FMath::Abs(1.f - Plane.SizeSquared()) < KINDA_SMALL_NUMBER && TEXT("not normalized"));
}

} // namespace Math
} // namespace UE

UE_DECLARE_LWC_TYPE(MirrorMatrix, 44);

template<> struct TIsUECoreVariant<FMirrorMatrix44f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FMirrorMatrix44d> { enum { Value = true }; };