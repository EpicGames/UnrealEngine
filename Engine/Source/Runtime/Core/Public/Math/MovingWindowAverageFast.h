// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "Containers/StaticArray.h"

/**
 * This class calculates a moving window average. Its designed to be used with floats or doubles and
 * keeps track of the average with every value pushed so is ideal when there is a one to one or one to many relationship
 * between calls to PushValue() and GetMovingWindowAverage() respectively.
 */
template <typename T, int32 ArraySize>
class FMovingWindowAverageFast
{
public:
	[[nodiscard]] FMovingWindowAverageFast()
		: TotalValues(static_cast<T>(0))
		, AverageValue(static_cast<T>(0))
		, RemoveNextIdx(0)
		, NumValuesUsed(0)
	{
		static_assert(ArraySize > 0, "ArraySize must be greater than zero");
	}

	void PushValue(T Value)
	{
		T ValueRemoved = static_cast<T>(0);

		const T NumItemsPrev = static_cast<T>(NumValuesUsed);

		if (ArraySize == NumValuesUsed)
		{
			ValueRemoved = ValuesArray[RemoveNextIdx];
			ValuesArray[RemoveNextIdx] = Value;
			RemoveNextIdx = (RemoveNextIdx + 1) % ArraySize;
		}
		else
		{
			ValuesArray[NumValuesUsed] = Value;
			++NumValuesUsed;
		}

		const T MovingWindowItemsNumCur = static_cast<T>(NumValuesUsed);
		TotalValues = TotalValues - ValueRemoved + Value;
		AverageValue = TotalValues / MovingWindowItemsNumCur;
	}

	[[nodiscard]] T GetAverage() const
	{
		return AverageValue;
	}

private:
	TStaticArray<T, ArraySize> ValuesArray;

	T TotalValues;
	T AverageValue;

	/** The array Index of the next item to remove when the moving window is full */
	int32 RemoveNextIdx;
	int32 NumValuesUsed;
};
