// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/* Numeric constants
 *****************************************************************************/

#define MIN_uint8		((uint8)	0x00)
#define	MIN_uint16		((uint16)	0x0000)
#define	MIN_uint32		((uint32)	0x00000000)
#define MIN_uint64		((uint64)	0x0000000000000000)
#define MIN_int8		((int8)		-128)
#define MIN_int16		((int16)	-32768)
#define MIN_int32		((int32)	0x80000000)
#define MIN_int64		((int64)	0x8000000000000000)

#define MAX_uint8		((uint8)	0xff)
#define MAX_uint16		((uint16)	0xffff)
#define MAX_uint32		((uint32)	0xffffffff)
#define MAX_uint64		((uint64)	0xffffffffffffffff)
#define MAX_int8		((int8)		0x7f)
#define MAX_int16		((int16)	0x7fff)
#define MAX_int32		((int32)	0x7fffffff)
#define MAX_int64		((int64)	0x7fffffffffffffff)

#define MIN_flt			(1.175494351e-38F)			/* min positive value */
#define MAX_flt			(3.402823466e+38F)
#define MIN_dbl			(2.2250738585072014e-308)	/* min positive value */
#define MAX_dbl			(1.7976931348623158e+308)	


/* Numeric type traits
 *****************************************************************************/

/**
 * Helper class to map a numeric type to its limits
 */
template <typename NumericType>
struct TNumericLimits;


/**
 * Numeric limits for const types
 */
template <typename NumericType>
struct TNumericLimits<const NumericType> 
	: public TNumericLimits<NumericType>
{ };


/**
 * Numeric limits for volatile types
 */
template <typename NumericType>
struct TNumericLimits<volatile NumericType> 
	: public TNumericLimits<NumericType>
{ };


/**
 * Numeric limits for const volatile types
 */
template <typename NumericType>
struct TNumericLimits<const volatile NumericType> 
	: public TNumericLimits<NumericType>
{ };


template<>
struct TNumericLimits<uint8> 
{
	typedef uint8 NumericType;
	
	[[nodiscard]] static constexpr NumericType Min()
	{
		return MIN_uint8;
	}

	[[nodiscard]] static constexpr NumericType Max()
	{
		return MAX_uint8;
	}

	[[nodiscard]] static constexpr NumericType Lowest()
	{
		return Min();
	}
};


template<>
struct TNumericLimits<uint16> 
{
	typedef uint16 NumericType;
	
	[[nodiscard]] static constexpr NumericType Min()
	{
		return MIN_uint16;
	}

	[[nodiscard]] static constexpr NumericType Max()
	{
		return MAX_uint16;
	}

	[[nodiscard]] static constexpr NumericType Lowest()
	{
		return Min();
	}
};


template<>
struct TNumericLimits<uint32> 
{
	typedef uint32 NumericType;
	
	[[nodiscard]] static constexpr NumericType Min()
	{
		return MIN_uint32;
	}

	[[nodiscard]] static constexpr NumericType Max()
	{
		return MAX_uint32;
	}
	
	[[nodiscard]] static constexpr NumericType Lowest()
	{
		return Min();
	}
};


template<>
struct TNumericLimits<uint64> 
{
	typedef uint64 NumericType;
	
	[[nodiscard]] static constexpr NumericType Min()
	{
		return MIN_uint64;
	}

	[[nodiscard]] static constexpr NumericType Max()
	{
		return MAX_uint64;
	}

	[[nodiscard]] static constexpr NumericType Lowest()
	{
		return Min();
	}
};


template<>
struct TNumericLimits<int8> 
{
	typedef int8 NumericType;
	
	[[nodiscard]] static constexpr NumericType Min()
	{
		return MIN_int8;
	}

	[[nodiscard]] static constexpr NumericType Max()
	{
		return MAX_int8;
	}

	[[nodiscard]] static constexpr NumericType Lowest()
	{
		return Min();
	}
};


template<>
struct TNumericLimits<int16> 
{
	typedef int16 NumericType;

	[[nodiscard]] static constexpr NumericType Min()
	{
		return MIN_int16;
	}

	[[nodiscard]] static constexpr NumericType Max()
	{
		return MAX_int16;
	}

	[[nodiscard]] static constexpr NumericType Lowest()
	{
		return Min();
	}
};


template<>
struct TNumericLimits<int32> 
{
	typedef int32 NumericType;

	[[nodiscard]] static constexpr NumericType Min()
	{
		return MIN_int32;
	}

	[[nodiscard]] static constexpr NumericType Max()
	{
		return MAX_int32;
	}

	[[nodiscard]] static constexpr NumericType Lowest()
	{
		return Min();
	}
};


template<>
struct TNumericLimits<int64> 
{
	typedef int64 NumericType;
	
	[[nodiscard]] static constexpr NumericType Min()
	{
		return MIN_int64;
	}

	[[nodiscard]] static constexpr NumericType Max()
	{
		return MAX_int64;
	}

	[[nodiscard]] static constexpr NumericType Lowest()
	{
		return Min();
	}
};


template<>
struct TNumericLimits<float> 
{
	typedef float NumericType;

	[[nodiscard]] static constexpr NumericType Min()
	{
		return MIN_flt;
	}

	[[nodiscard]] static constexpr NumericType Max()
	{
		return MAX_flt;
	}

	[[nodiscard]] static constexpr NumericType Lowest()
	{
		return -Max();
	}
};


template<>
struct TNumericLimits<double> 
{
	typedef double NumericType;

	[[nodiscard]] static constexpr NumericType Min()
	{
		return MIN_dbl;
	}

	[[nodiscard]] static constexpr NumericType Max()
	{
		return MAX_dbl;
	}

	[[nodiscard]] static constexpr NumericType Lowest()
	{
		return -Max();
	}
};
