// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector.h"
#include "Math/Interval.h"

/**
 * Structure for arbitrarily oriented boxes (not necessarily axis-aligned).
 */
struct FOrientedBox
{
	/** Holds the center of the box. */
	FVector Center;

	/** Holds the x-axis vector of the box. Must be a unit vector. */
	FVector AxisX;
	
	/** Holds the y-axis vector of the box. Must be a unit vector. */
	FVector AxisY;
	
	/** Holds the z-axis vector of the box. Must be a unit vector. */
	FVector AxisZ;

	/** Holds the extent of the box along its x-axis. */
	FVector::FReal ExtentX;
	
	/** Holds the extent of the box along its y-axis. */
	FVector::FReal ExtentY;

	/** Holds the extent of the box along its z-axis. */
	FVector::FReal ExtentZ;

public:

	/**
	 * Default constructor.
	 *
	 * Constructs a unit-sized, origin-centered box with axes aligned to the coordinate system.
	 */
	[[nodiscard]] FOrientedBox()
		: Center(0.0f)
		, AxisX(1.0f, 0.0f, 0.0f)
		, AxisY(0.0f, 1.0f, 0.0f)
		, AxisZ(0.0f, 0.0f, 1.0f)
		, ExtentX(1.0f)
		, ExtentY(1.0f)
		, ExtentZ(1.0f)
	{ }

public:

	/**
	 * Fills in the Verts array with the eight vertices of the box.
	 *
	 * @param Verts The array to fill in with the vertices.
	 */
	inline void CalcVertices(FVector* Verts) const;

	/**
	 * Finds the projection interval of the box when projected onto Axis.
	 *
	 * @param Axis The unit vector defining the axis to project the box onto.
	 */
	[[nodiscard]] inline FFloatInterval Project(const FVector& Axis) const;
};


/* FOrientedBox inline functions
 *****************************************************************************/

inline void FOrientedBox::CalcVertices( FVector* Verts ) const
{
	const float Signs[] = { -1.0f, 1.0f };

	for (int32 i = 0; i < 2; i++)
	{
		for (int32 j = 0; j < 2; j++)
		{
			for (int32 k = 0; k < 2; k++)
			{
				*Verts++ = Center + (Signs[i] * ExtentX) * AxisX + (Signs[j] * ExtentY) * AxisY + (Signs[k] * ExtentZ) * AxisZ;
			}
		}
	}
}

inline FFloatInterval FOrientedBox::Project( const FVector& Axis ) const
{
	// Consider:
	//   max { dot(Center +- ExtentX * AxisX +- ExtentY * AxisY +- ExtentZ * AxisZ, Axis) }
	//  =max { dot(Center, Axis) +- ExtentX * dot(AxisX, Axis) +- ExtentY * dot(AxisY, Axis) +- ExtentZ * dot(AxisZ, Axis) }
	//
	// These individual terms can be maximized separately and are clearly maximal when their effective signs
	// are all positive. Analogous for the min with all-negative signs.

	// Calculate the projections of the box center and the extent-scaled axes.
	FVector::FReal ProjectedCenter = Axis | Center;
	FVector::FReal AbsProjectedAxisX = FMath::Abs(ExtentX * (Axis | AxisX));
	FVector::FReal AbsProjectedAxisY = FMath::Abs(ExtentY * (Axis | AxisY));
	FVector::FReal AbsProjectedAxisZ = FMath::Abs(ExtentZ * (Axis | AxisZ));

	FVector::FReal AbsProjectedExtent = AbsProjectedAxisX + AbsProjectedAxisY + AbsProjectedAxisZ;
	return FFloatInterval {
		float(ProjectedCenter - AbsProjectedExtent),
		float(ProjectedCenter + AbsProjectedExtent)
	};
}

template <> struct TIsPODType<FOrientedBox> { enum { Value = true }; };
