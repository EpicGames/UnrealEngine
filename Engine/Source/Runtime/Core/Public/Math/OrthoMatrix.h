// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Plane.h"
#include "Math/Matrix.h"

namespace UE {
namespace Math {


template<typename T>
struct TOrthoMatrix
	: public TMatrix<T>
{
public:

	/**
	 * Constructor
	 *
	 * @param Width view space width
	 * @param Height view space height
	 * @param ZScale scale in the Z axis
	 * @param ZOffset offset in the Z axis
	 */
	TOrthoMatrix(T Width,T Height,T ZScale,T ZOffset);
	
	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TOrthoMatrix(const TOrthoMatrix<FArg>& From)
		: TMatrix<T>(From)
	{
	}
};


template<typename T>
struct TReversedZOrthoMatrix : public TMatrix<T>
{
public:
	TReversedZOrthoMatrix(T Width,T Height,T ZScale,T ZOffset);
	TReversedZOrthoMatrix(T Left, T Right, T Bottom, T Top, T ZScale, T ZOffset);

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TReversedZOrthoMatrix(const TReversedZOrthoMatrix<FArg>& From)
		: TMatrix<T>(From)
	{
	}
};

template<typename T>
inline TOrthoMatrix<T>::TOrthoMatrix(T Width, T Height, T ZScale, T ZOffset)
	: TMatrix<T>(
		TPlane<T>((Width != 0.0f) ? (1.0f / Width) : 1.0f, 0.0f, 0.0f, 0.0f),
		TPlane<T>(0.0f, (Height != 0.0f) ? (1.0f / Height) : 1.f, 0.0f, 0.0f),
		TPlane<T>(0.0f, 0.0f, ZScale, 0.0f),
		TPlane<T>(0.0f, 0.0f, ZOffset * ZScale, 1.0f)
	)
{ }


template<typename T>
inline TReversedZOrthoMatrix<T>::TReversedZOrthoMatrix(T Width, T Height, T ZScale, T ZOffset)
	: TMatrix<T>(
		TPlane<T>((Width != 0.0f) ? (1.0f / Width) : 1.0f, 0.0f, 0.0f, 0.0f),
		TPlane<T>(0.0f, (Height != 0.0f) ? (1.0f / Height) : 1.f, 0.0f, 0.0f),
		TPlane<T>(0.0f, 0.0f, -ZScale, 0.0f),
		TPlane<T>(0.0f, 0.0f, 1.0f - ZOffset * ZScale, 1.0f)
	)
{ }

template<typename T>
inline TReversedZOrthoMatrix<T>::TReversedZOrthoMatrix(T Left, T Right, T Bottom, T Top, T ZScale, T ZOffset)
	: TMatrix<T>(
		TPlane<T>(1.0f / (Right - Left), 0.0f, 0.0f, 0.0f),
		TPlane<T>(0.0f, 1.0f / (Top - Bottom), 0.0f, 0.0f),
		TPlane<T>(0.0f, 0.0f, -ZScale, 0.0f),
		TPlane<T>((Left + Right) / (Left - Right), (Top + Bottom) / (Bottom - Top), 1.0f - ZOffset * ZScale, 1.0f)
	)
{ }

 } // namespace Math
 } // namespace UE

UE_DECLARE_LWC_TYPE(OrthoMatrix, 44);
UE_DECLARE_LWC_TYPE(ReversedZOrthoMatrix, 44);

template<> struct TIsUECoreVariant<FOrthoMatrix44f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FOrthoMatrix44d> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FReversedZOrthoMatrix44f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FReversedZOrthoMatrix44d> { enum { Value = true }; };