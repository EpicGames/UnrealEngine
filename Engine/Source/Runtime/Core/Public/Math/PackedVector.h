// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Color.h"
#include "Templates/TypeCompatibleBytes.h"

/** 
 * 3 component vector corresponding to DXGI_FORMAT_R11G11B10_FLOAT. 
 * Conversion code from XMFLOAT3PK in DirectXPackedVector.h
 */
class FFloat3Packed
{
public: 
	union
    {
        struct
        {
            uint32_t xm : 6; // x-mantissa
            uint32_t xe : 5; // x-exponent
            uint32_t ym : 6; // y-mantissa
            uint32_t ye : 5; // y-exponent
            uint32_t zm : 5; // z-mantissa
            uint32_t ze : 5; // z-exponent
        };
        uint32_t v;
    };

	[[nodiscard]] FFloat3Packed() {}

	[[nodiscard]] CORE_API explicit FFloat3Packed(const FLinearColor& Src);

	[[nodiscard]] CORE_API FLinearColor ToLinearColor() const;
};

/** 
 * 4 component vector corresponding to PF_R8G8B8A8_SNORM. 
 * This differs from FColor which is BGRA.
 */
class FFixedRGBASigned8
{
public: 
	union
    {
        struct
        {
			int8 R;
			int8 G;
			int8 B;
			int8 A;
        };
        uint32 Packed;
    };

	FFixedRGBASigned8() {}

	explicit CORE_API FFixedRGBASigned8(const FLinearColor& Src);

	[[nodiscard]] CORE_API FLinearColor ToLinearColor() const;
};

/**
 * 3 component vector corresponding to PF_R9G9B9EXP5.
 */
class FFloat3PackedSE
{
public:

	union
	{
		struct
		{
			uint32 RMantissa : 9;
			uint32 GMantissa : 9;
			uint32 BMantissa : 9;
			uint32 SharedExponent : 5;
		};
		uint32 EncodedValue;
	};

	FFloat3PackedSE() {}

	explicit CORE_API FFloat3PackedSE(const FLinearColor& Src);
	explicit FFloat3PackedSE(uint32 InEncodedValue) : EncodedValue(InEncodedValue) {}

	[[nodiscard]] CORE_API FLinearColor ToLinearColor() const;
};

