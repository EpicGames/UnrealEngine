// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Plane.h"
#include "Math/Matrix.h"

namespace UE {
namespace Math {

template<typename T>
struct TPerspectiveMatrix
	: public TMatrix<T>
{
public:

// Note: the value of this must match the mirror in Common.usf!
#define Z_PRECISION	0.0f

	/**
	 * Constructor
	 *
	 * @param HalfFOVX Half FOV in the X axis
	 * @param HalfFOVY Half FOV in the Y axis
	 * @param MultFOVX multiplier on the X axis
	 * @param MultFOVY multiplier on the y axis
	 * @param MinZ distance to the near Z plane
	 * @param MaxZ distance to the far Z plane
	 */
	TPerspectiveMatrix(T HalfFOVX, T HalfFOVY, T MultFOVX, T MultFOVY, T MinZ, T MaxZ);

	/**
	 * Constructor
	 *
	 * @param HalfFOV half Field of View in the Y direction
	 * @param Width view space width
	 * @param Height view space height
	 * @param MinZ distance to the near Z plane
	 * @param MaxZ distance to the far Z plane
	 * @note that the FOV you pass in is actually half the FOV, unlike most perspective matrix functions (D3DXMatrixPerspectiveFovLH).
	 */
	TPerspectiveMatrix(T HalfFOV, T Width, T Height, T MinZ, T MaxZ);

	/**
	 * Constructor
	 *
	 * @param HalfFOV half Field of View in the Y direction
	 * @param Width view space width
	 * @param Height view space height
	 * @param MinZ distance to the near Z plane
	 * @note that the FOV you pass in is actually half the FOV, unlike most perspective matrix functions (D3DXMatrixPerspectiveFovLH).
	 */
	TPerspectiveMatrix(T HalfFOV, T Width, T Height, T MinZ);

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TPerspectiveMatrix(const TPerspectiveMatrix<FArg>& From)
		: TMatrix<T>(From)
	{
	}
};


template<typename T>
struct TReversedZPerspectiveMatrix : public TMatrix<T>
{
public:
	TReversedZPerspectiveMatrix(T HalfFOVX, T HalfFOVY, T MultFOVX, T MultFOVY, T MinZ, T MaxZ);
	TReversedZPerspectiveMatrix(T HalfFOV, T Width, T Height, T MinZ, T MaxZ);
	TReversedZPerspectiveMatrix(T HalfFOV, T Width, T Height, T MinZ);
	
	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TReversedZPerspectiveMatrix(const TReversedZPerspectiveMatrix<FArg>& From)
		: TMatrix<T>(From)
	{
	}
};


#ifdef _MSC_VER
#pragma warning (push)
// Disable possible division by 0 warning
#pragma warning (disable : 4723)
#endif


template<typename T>
inline TPerspectiveMatrix<T>::TPerspectiveMatrix(T HalfFOVX, T HalfFOVY, T MultFOVX, T MultFOVY, T MinZ, T MaxZ)
	: TMatrix<T>(
		TPlane<T>(MultFOVX / FMath::Tan(HalfFOVX),	0.0f,								0.0f,																	0.0f),
		TPlane<T>(0.0f,								MultFOVY / FMath::Tan(HalfFOVY),	0.0f,																	0.0f),
		TPlane<T>(0.0f,								0.0f,								((MinZ == MaxZ) ? (1.0f - Z_PRECISION) : MaxZ / (MaxZ - MinZ)),			1.0f),
		TPlane<T>(0.0f,								0.0f,								-MinZ * ((MinZ == MaxZ) ? (1.0f - Z_PRECISION) : MaxZ / (MaxZ - MinZ)),	0.0f)
	)
{ }


template<typename T>
inline TPerspectiveMatrix<T>::TPerspectiveMatrix(T HalfFOV, T Width, T Height, T MinZ, T MaxZ)
	: TMatrix<T>(
		TPlane<T>(1.0f / FMath::Tan(HalfFOV),	0.0f,									0.0f,							0.0f),
		TPlane<T>(0.0f,							Width / FMath::Tan(HalfFOV) / Height,	0.0f,							0.0f),
		TPlane<T>(0.0f,							0.0f,									((MinZ == MaxZ) ? (1.0f - Z_PRECISION) : MaxZ / (MaxZ - MinZ)),			1.0f),
		TPlane<T>(0.0f,							0.0f,									-MinZ * ((MinZ == MaxZ) ? (1.0f - Z_PRECISION) : MaxZ / (MaxZ - MinZ)),	0.0f)
	)
{ }


template<typename T>
inline TPerspectiveMatrix<T>::TPerspectiveMatrix(T HalfFOV, T Width, T Height, T MinZ)
	: TMatrix<T>(
		TPlane<T>(1.0f / FMath::Tan(HalfFOV),	0.0f,									0.0f,							0.0f),
		TPlane<T>(0.0f,							Width / FMath::Tan(HalfFOV) / Height,	0.0f,							0.0f),
		TPlane<T>(0.0f,							0.0f,									(1.0f - Z_PRECISION),			1.0f),
		TPlane<T>(0.0f,							0.0f,									-MinZ * (1.0f - Z_PRECISION),	0.0f)
	)
{ }


template<typename T>
inline TReversedZPerspectiveMatrix<T>::TReversedZPerspectiveMatrix(T HalfFOVX, T HalfFOVY, T MultFOVX, T MultFOVY, T MinZ, T MaxZ)
	: TMatrix<T>(
		TPlane<T>(MultFOVX / FMath::Tan(HalfFOVX),	0.0f,								0.0f,													0.0f),
		TPlane<T>(0.0f,								MultFOVY / FMath::Tan(HalfFOVY),	0.0f,													0.0f),
		TPlane<T>(0.0f,								0.0f,								((MinZ == MaxZ) ? 0.0f : MinZ / (MinZ - MaxZ)),			1.0f),
		TPlane<T>(0.0f,								0.0f,								((MinZ == MaxZ) ? MinZ : -MaxZ * MinZ / (MinZ - MaxZ)),	0.0f)
	)
{ }


template<typename T>
inline TReversedZPerspectiveMatrix<T>::TReversedZPerspectiveMatrix(T HalfFOV, T Width, T Height, T MinZ, T MaxZ)
	: TMatrix<T>(
		TPlane<T>(1.0f / FMath::Tan(HalfFOV),	0.0f,									0.0f,													0.0f),
		TPlane<T>(0.0f,							Width / FMath::Tan(HalfFOV) / Height,	0.0f,													0.0f),
		TPlane<T>(0.0f,							0.0f,									((MinZ == MaxZ) ? 0.0f : MinZ / (MinZ - MaxZ)),			1.0f),
		TPlane<T>(0.0f,							0.0f,									((MinZ == MaxZ) ? MinZ : -MaxZ * MinZ / (MinZ - MaxZ)),	0.0f)
	)
{ }


template<typename T>
inline TReversedZPerspectiveMatrix<T>::TReversedZPerspectiveMatrix(T HalfFOV, T Width, T Height, T MinZ)
	: TMatrix<T>(
		TPlane<T>(1.0f / FMath::Tan(HalfFOV),	0.0f,									0.0f, 0.0f),
		TPlane<T>(0.0f,							Width / FMath::Tan(HalfFOV) / Height,	0.0f, 0.0f),
		TPlane<T>(0.0f,							0.0f,									0.0f, 1.0f),
		TPlane<T>(0.0f,							0.0f,									MinZ, 0.0f)
	)
{ }

#ifdef _MSC_VER
#pragma warning (pop)
#endif


} // namespace Math
} // namespace UE

UE_DECLARE_LWC_TYPE(PerspectiveMatrix, 44);
UE_DECLARE_LWC_TYPE(ReversedZPerspectiveMatrix, 44);

template<> struct TIsUECoreVariant<FPerspectiveMatrix44f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FPerspectiveMatrix44d> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FReversedZPerspectiveMatrix44f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FReversedZPerspectiveMatrix44d> { enum { Value = true }; };