// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/MathFwd.h" // IWYU pragma: export
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Misc/LargeWorldCoordinatesSerializer.h"
#include "UObject/ObjectVersion.h"

#ifdef _MSC_VER
#pragma warning (push)
// Ensure template functions don't generate shadowing warnings against global variables at the point of instantiation.
#pragma warning (disable : 4459)
#endif

/**
 * Structure for three dimensional planes.
 *
 * Stores the coeffecients as Xx+Yy+Zz=W.
 * Note that this is different from many other Plane classes that use Xx+Yy+Zz+W=0.
 */

namespace UE
{
namespace Math
{

template<typename T>
struct alignas(16) TPlane
	: public TVector<T>
{
public:
	using FReal = T;
	using TVector<T>::X;
	using TVector<T>::Y;
	using TVector<T>::Z;

	/** The w-component. */
	T W;



#if ENABLE_NAN_DIAGNOSTIC
	inline void DiagnosticCheckNaN() const
	{
		if (TVector<T>::ContainsNaN() || !FMath::IsFinite(W))
		{
			logOrEnsureNanError(TEXT("FPlane contains NaN: %s W=%3.3f"), *TVector<T>::ToString(), W);
		}
	}
#else
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN() const {}
#endif

public:

	/** Default constructor (no initialization). */
	[[nodiscard]] TPlane() = default;

	/**
	 * Constructor.
	 *
	 * @param V 4D vector to set up plane.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TPlane(const TVector4<T>& V);

	/**
	 * Constructor.
	 *
	 * @param InX X-coefficient.
	 * @param InY Y-coefficient.
	 * @param InZ Z-coefficient.
	 * @param InW W-coefficient.
	 */
	[[nodiscard]] inline TPlane(T InX, T InY, T InZ, T InW);

	/**
	 * Constructor.
	 *
	 * @param InNormal Plane Normal Vector.
	 * @param InW Plane W-coefficient.
	 */
	[[nodiscard]] inline TPlane(TVector<T> InNormal, T InW);

	/**
	 * Constructor.
	 *
	 * @param InBase Base point in plane.
	 * @param InNormal Plane Normal Vector.
	 */
	[[nodiscard]] inline TPlane(TVector<T> InBase, const TVector<T>& InNormal);

	/**
	 * Constructor.
	 *
	 * @param A First point in the plane.
	 * @param B Second point in the plane.
	 * @param C Third point in the plane.
	 */
	[[nodiscard]] TPlane(TVector<T> A, TVector<T> B, TVector<T> C);

	/**
	 * Constructor
	 *
	 * @param EForceInit Force Init Enum.
	 */
	[[nodiscard]] explicit UE_FORCEINLINE_HINT TPlane(EForceInit);

	// Functions.

	/**
	 * Checks if this plane is valid (ie: if it has a non-zero normal).
	 *
	 * @return true if the plane is well-defined (has a non-zero normal), otherwise false.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValid() const;

	/**
	 * Get the origin of this plane.
	 *
	 * @return The origin (base point) of this plane.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> GetOrigin() const;

	/**
	 * Get the normal of this plane.
	 *
	 * @return The normal of this plane.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const TVector<T>& GetNormal() const;


	/**
	 * Calculates distance between plane and a point.
	 *
	 * @param P The other point.
	 * @return The distance from the plane to the point. 0: Point is on the plane. >0: Point is in front of the plane. <0: Point is behind the plane.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T PlaneDot(const TVector<T>& P) const;

	/**
	 * Normalize this plane in-place if it is larger than a given tolerance. Leaves it unchanged if not.
	 *
	 * @param Tolerance Minimum squared length of vector for normalization.
	 * @return true if the plane was normalized correctly, false otherwise.
	 */
	bool Normalize(T Tolerance = UE_SMALL_NUMBER);

	/**
	 * Get a flipped version of the plane.
	 *
	 * @return A flipped version of the plane.
	 */
	[[nodiscard]] TPlane<T> Flip() const;

	/**
	 * Get the result of transforming the plane by a Matrix.
	 *
	 * @param M The matrix to transform plane with.
	 * @return The result of transform.
	 */
	[[nodiscard]] TPlane<T> TransformBy(const TMatrix<T>& M) const;

	/**
	 * You can optionally pass in the matrices transpose-adjoint, which save it recalculating it.
	 * MSM: If we are going to save the transpose-adjoint we should also save the more expensive
	 * determinant.
	 *
	 * @param M The Matrix to transform plane with.
	 * @param DetM Determinant of Matrix.
	 * @param TA Transpose-adjoint of Matrix.
	 * @return The result of transform.
	 */
	[[nodiscard]] TPlane<T> TransformByUsingAdjointT(const TMatrix<T>& M, T DetM, const TMatrix<T>& TA) const;

	/**
	 * Get the result of translating the plane by the given offset
	 *
	 * @param V The translation amount
	 * @return The result of transform.
	 */
	[[nodiscard]] TPlane<T> TranslateBy(const TVector<T>& V) const;

	/**
	 * Check if two planes are identical.
	 *
	 * @param V The other plane.
	 * @return true if planes are identical, otherwise false.
	 */
	[[nodiscard]] bool operator==(const TPlane<T>& V) const;

	/**
	 * Check if two planes are different.
	 *
	 * @param V The other plane.
	 * @return true if planes are different, otherwise false.
	 */
	[[nodiscard]] bool operator!=(const TPlane<T>& V) const;

	/**
	 * Checks whether two planes are equal within specified tolerance.
	 *
	 * @param V The other plane.
	 * @param Tolerance Error Tolerance.
	 * @return true if the two planes are equal within specified tolerance, otherwise false.
	 */
	[[nodiscard]] bool Equals(const TPlane<T>& V, T Tolerance = UE_KINDA_SMALL_NUMBER) const;

	/**
	 * Calculates dot product of two planes.
	 *
	 * @param V The other plane.
	 * @return The dot product.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T operator|(const TPlane<T>& V) const;

	/**
	 * Gets result of adding a plane to this.
	 *
	 * @param V The other plane.
	 * @return The result of adding a plane to this.
	 */
	[[nodiscard]] TPlane<T> operator+(const TPlane<T>& V) const;

	/**
	 * Gets result of subtracting a plane from this.
	 *
	 * @param V The other plane.
	 * @return The result of subtracting a plane from this.
	 */
	[[nodiscard]] TPlane<T> operator-(const TPlane<T>& V) const;

	/**
	 * Gets result of dividing a plane.
	 *
	 * @param Scale What to divide by.
	 * @return The result of division.
	 */
	[[nodiscard]] TPlane<T> operator/(T Scale) const;

	/**
	 * Gets result of scaling a plane.
	 *
	 * @param Scale The scaling factor.
	 * @return The result of scaling.
	 */
	[[nodiscard]] TPlane<T> operator*(T Scale) const;

	/**
	 * Gets result of multiplying a plane with this.
	 *
	 * @param V The other plane.
	 * @return The result of multiplying a plane with this.
	 */
	[[nodiscard]] TPlane<T> operator*(const TPlane<T>& V) const;

	/**
	 * Add another plane to this.
	 *
	 * @param V The other plane.
	 * @return Copy of plane after addition.
	 */
	TPlane<T> operator+=(const TPlane<T>& V);

	/**
	 * Subtract another plane from this.
	 *
	 * @param V The other plane.
	 * @return Copy of plane after subtraction.
	 */
	TPlane<T> operator-=(const TPlane<T>& V);

	/**
	 * Scale this plane.
	 *
	 * @param Scale The scaling factor.
	 * @return Copy of plane after scaling.
	 */
	TPlane<T> operator*=(T Scale);

	/**
	 * Multiply another plane with this.
	 *
	 * @param V The other plane.
	 * @return Copy of plane after multiplication.
	 */
	TPlane<T> operator*=(const TPlane<T>& V);

	/**
	 * Divide this plane.
	 *
	 * @param V What to divide by.
	 * @return Copy of plane after division.
	 */
	TPlane<T> operator/=(T V);

	bool Serialize(FArchive& Ar)
	{
		//if (Ar.UEVer() >= VER_UE4_ADDED_NATIVE_SERIALIZATION_FOR_IMMUTABLE_STRUCTURES)
		{
			Ar << (TPlane<T>&)*this;
			return true;
		}
		//return false;
	}

	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		if constexpr (std::is_same_v<T, float>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Plane, Plane4f, Plane4d);
		}
		else if constexpr (std::is_same_v<T, double>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Plane, Plane4d, Plane4f);
		}
		else
		{
			static_assert(sizeof(T) == 0, "Unimplemented");
			return false;
		}
	}

	/**
	 * Serializes the vector compressed for e.g. network transmission.
	 * @param Ar Archive to serialize to/ from.
	 * @return false to allow the ordinary struct code to run (this never happens).
	 */
	bool NetSerialize(FArchive& Ar, class UPackageMap*, bool& bOutSuccess)
	{
		if (Ar.IsLoading())
		{
			int16 iX, iY, iZ, iW;
			Ar << iX << iY << iZ << iW;
			*this = TPlane<T>(iX, iY, iZ, iW);
		}
		else
		{
			int16 iX((int16)FMath::RoundToInt(X));
			int16 iY((int16)FMath::RoundToInt(Y));
			int16 iZ((int16)FMath::RoundToInt(Z));
			int16 iW((int16)FMath::RoundToInt(W));
			Ar << iX << iY << iZ << iW;
		}
		bOutSuccess = true;
		return true;
	}

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TPlane(const TPlane<FArg>& From)
		: TPlane<T>((T)From.X, (T)From.Y, (T)From.Z, (T)From.W)
	{
	}
};


/**
 * Serializer.
 *
 * @param Ar Serialization Archive.
 * @param P Plane to serialize.
 * @return Reference to Archive after serialization.
 */
inline FArchive& operator<<(FArchive& Ar, TPlane<float>& P)
{
	Ar << (TVector<float>&)P << P.W;
	P.DiagnosticCheckNaN();
	return Ar;
}

/**
 * Serializer.
 *
 * @param Ar Serialization Archive.
 * @param P Plane to serialize.
 * @return Reference to Archive after serialization.
 */
inline FArchive& operator<<(FArchive& Ar, TPlane<double>& P)
{
	Ar << (TVector<double>&)P;

	if (Ar.UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
	{
		Ar << P.W;
	}
	else
	{
		checkf(Ar.IsLoading(), TEXT("float -> double conversion applied outside of load!"));
		// Stored as floats, so serialize float and copy.
		float SW;
		Ar << SW;
		P.W = SW;
	}

	P.DiagnosticCheckNaN();
	return Ar;
}


/* TPlane inline functions
 *****************************************************************************/

template<typename T>
UE_FORCEINLINE_HINT TPlane<T>::TPlane(const TVector4<T>& V)
	: TVector<T>(V)
	, W(V.W)
{}


template<typename T>
inline TPlane<T>::TPlane(T InX, T InY, T InZ, T InW)
	: TVector<T>(InX, InY, InZ)
	, W(InW)
{
	DiagnosticCheckNaN();
}


template<typename T>
inline TPlane<T>::TPlane(TVector<T> InNormal, T InW)
	: TVector<T>(InNormal), W(InW)
{
	DiagnosticCheckNaN();
}

template<typename T>
inline TPlane<T>::TPlane(TVector<T> InBase, const TVector<T>& InNormal)
	: TVector<T>(InNormal)
	, W(InBase | InNormal)
{
	DiagnosticCheckNaN();
}


template<typename T>
inline TPlane<T>::TPlane(TVector<T> A, TVector<T> B, TVector<T> C)
	: TVector<T>(((B - A) ^ (C - A)).GetSafeNormal())
{
	W = A | (TVector<T>)(*this);
	DiagnosticCheckNaN();
}


template<typename T>
UE_FORCEINLINE_HINT TPlane<T>::TPlane(EForceInit)
	: TVector<T>(ForceInit), W(0.f)
{}

template<typename T>
UE_FORCEINLINE_HINT bool TPlane<T>::IsValid() const
{
	return !this->IsNearlyZero();
}

template<typename T>
UE_FORCEINLINE_HINT const TVector<T>& TPlane<T>::GetNormal() const
{
	return *this;
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TPlane<T>::GetOrigin() const
{
	return GetNormal() * W;
}

template<typename T>
UE_FORCEINLINE_HINT T TPlane<T>::PlaneDot(const TVector<T>& P) const
{
	return X * P.X + Y * P.Y + Z * P.Z - W;
}

template<typename T>
inline bool TPlane<T>::Normalize(T Tolerance)
{
	const T SquareSum = X * X + Y * Y + Z * Z;
	if (SquareSum > Tolerance)
	{
		const T Scale = FMath::InvSqrt(SquareSum);
		X *= Scale; Y *= Scale; Z *= Scale; W *= Scale;
		return true;
	}
	return false;
}

template<typename T>
UE_FORCEINLINE_HINT TPlane<T> TPlane<T>::Flip() const
{
	return TPlane<T>(-X, -Y, -Z, -W);
}

template<typename T>
UE_FORCEINLINE_HINT TPlane<T> TPlane<T>::TranslateBy(const TVector<T>& V) const
{
	return TPlane<T>(GetOrigin() + V, GetNormal());
}

template<typename T>
UE_FORCEINLINE_HINT bool TPlane<T>::operator==(const TPlane<T>& V) const
{
	return (X == V.X) && (Y == V.Y) && (Z == V.Z) && (W == V.W);
}


template<typename T>
UE_FORCEINLINE_HINT bool TPlane<T>::operator!=(const TPlane<T>& V) const
{
	return (X != V.X) || (Y != V.Y) || (Z != V.Z) || (W != V.W);
}


template<typename T>
UE_FORCEINLINE_HINT bool TPlane<T>::Equals(const TPlane<T>& V, T Tolerance) const
{
	return (FMath::Abs(X - V.X) < Tolerance) && (FMath::Abs(Y - V.Y) < Tolerance) && (FMath::Abs(Z - V.Z) < Tolerance) && (FMath::Abs(W - V.W) < Tolerance);
}


template<typename T>
UE_FORCEINLINE_HINT T TPlane<T>::operator|(const TPlane<T>& V) const
{
	return X * V.X + Y * V.Y + Z * V.Z + W * V.W;
}


template<typename T>
UE_FORCEINLINE_HINT TPlane<T> TPlane<T>::operator+(const TPlane<T>& V) const
{
	return TPlane<T>(X + V.X, Y + V.Y, Z + V.Z, W + V.W);
}


template<typename T>
UE_FORCEINLINE_HINT TPlane<T> TPlane<T>::operator-(const TPlane<T>& V) const
{
	return TPlane<T>(X - V.X, Y - V.Y, Z - V.Z, W - V.W);
}


template<typename T>
inline TPlane<T> TPlane<T>::operator/(T Scale) const
{
	const T RScale = 1 / Scale;
	return TPlane<T>(X * RScale, Y * RScale, Z * RScale, W * RScale);
}


template<typename T>
UE_FORCEINLINE_HINT TPlane<T> TPlane<T>::operator*(T Scale) const
{
	return TPlane<T>(X * Scale, Y * Scale, Z * Scale, W * Scale);
}


template<typename T>
UE_FORCEINLINE_HINT TPlane<T> TPlane<T>::operator*(const TPlane<T>& V) const
{
	return TPlane<T>(X * V.X, Y * V.Y, Z * V.Z, W * V.W);
}


template<typename T>
inline TPlane<T> TPlane<T>::operator+=(const TPlane<T>& V)
{
	X += V.X; Y += V.Y; Z += V.Z; W += V.W;
	return *this;
}


template<typename T>
inline TPlane<T> TPlane<T>::operator-=(const TPlane<T>& V)
{
	X -= V.X; Y -= V.Y; Z -= V.Z; W -= V.W;
	return *this;
}


template<typename T>
inline TPlane<T> TPlane<T>::operator*=(T Scale)
{
	X *= Scale; Y *= Scale; Z *= Scale; W *= Scale;
	return *this;
}


template<typename T>
inline TPlane<T> TPlane<T>::operator*=(const TPlane<T>& V)
{
	X *= V.X; Y *= V.Y; Z *= V.Z; W *= V.W;
	return *this;
}


template<typename T>
inline TPlane<T> TPlane<T>::operator/=(T V)
{
	const T RV = 1 / V;
	X *= RV; Y *= RV; Z *= RV; W *= RV;
	return *this;
}


/* TVector inline functions
 *****************************************************************************/

template<typename T>
inline TVector<T> TVector<T>::MirrorByPlane(const TPlane<T>& Plane) const
{
	return *this - Plane * (2.f * Plane.PlaneDot(*this));
}

template<typename T>
inline TVector<T> TVector<T>::PointPlaneProject(const TVector<T>& Point, const TPlane<T>& Plane)
{
	//Find the distance of X from the plane
	//Add the distance back along the normal from the point
	return Point - Plane.PlaneDot(Point) * Plane;
}

template<typename T>
inline TVector<T> TVector<T>::PointPlaneProject(const TVector<T>& Point, const TVector<T>& A, const TVector<T>& B, const TVector<T>& C)
{
	//Compute the plane normal from ABC
	TPlane<T> Plane(A, B, C);

	//Find the distance of X from the plane
	//Add the distance back along the normal from the point
	return Point - Plane.PlaneDot(Point) * Plane;
}

}	// namespace UE::Math
}	// namespace UE


/* FMath inline functions
 *****************************************************************************/

template<typename T>
inline UE::Math::TVector<T> FMath::RayPlaneIntersection(const UE::Math::TVector<T>& RayOrigin, const UE::Math::TVector<T>& RayDirection, const UE::Math::TPlane<T>& Plane)
{
	using TVector = UE::Math::TVector<T>;
	const TVector PlaneNormal = TVector(Plane.X, Plane.Y, Plane.Z);
	const TVector PlaneOrigin = PlaneNormal * Plane.W;

	const T Distance = TVector::DotProduct((PlaneOrigin - RayOrigin), PlaneNormal) / TVector::DotProduct(RayDirection, PlaneNormal);
	return RayOrigin + RayDirection * Distance;
}

template<typename T>
inline T FMath::RayPlaneIntersectionParam(const UE::Math::TVector<T>& RayOrigin, const UE::Math::TVector<T>& RayDirection, const UE::Math::TPlane<T>& Plane)
{
	using TVector = UE::Math::TVector<T>;
	const TVector PlaneNormal = TVector(Plane.X, Plane.Y, Plane.Z);
	const TVector PlaneOrigin = PlaneNormal * Plane.W;

	return TVector::DotProduct((PlaneOrigin - RayOrigin), PlaneNormal) / TVector::DotProduct(RayDirection, PlaneNormal);
}

template<typename T>
inline UE::Math::TVector<T> FMath::LinePlaneIntersection
(
	const UE::Math::TVector<T>& Point1,
	const UE::Math::TVector<T>& Point2,
	const UE::Math::TPlane<T>& Plane
)
{
	return
		Point1
		+ (Point2 - Point1)
		* ((Plane.W - (Point1 | Plane)) / ((Point2 - Point1) | Plane));
}

template<typename T>
inline bool FMath::IntersectPlanes3(UE::Math::TVector<T>& I, const UE::Math::TPlane<T>& P1, const UE::Math::TPlane<T>& P2, const UE::Math::TPlane<T>& P3)
{
	// Compute determinant, the triple product P1|(P2^P3)==(P1^P2)|P3.
	const T Det = (P1 ^ P2) | P3;
	if (Square(Det) < Square(0.001f))
	{
		// Degenerate.
		I = UE::Math::TVector<T>::ZeroVector;
		return 0;
	}
	else
	{
		// Compute the intersection point, guaranteed valid if determinant is nonzero.
		I = (P1.W * (P2 ^ P3) + P2.W * (P3 ^ P1) + P3.W * (P1 ^ P2)) / Det;
	}
	return 1;
}

template<typename T>
inline bool FMath::IntersectPlanes2(UE::Math::TVector<T>& I, UE::Math::TVector<T>& D, const UE::Math::TPlane<T>& P1, const UE::Math::TPlane<T>& P2)
{
	// Compute line direction, perpendicular to both plane normals.
	D = P1 ^ P2;
	const T DD = D.SizeSquared();
	if (DD < Square(0.001f))
	{
		// Parallel or nearly parallel planes.
		D = I = UE::Math::TVector<T>::ZeroVector;
		return 0;
	}
	else
	{
		// Compute intersection.
		I = (P1.W * (P2 ^ D) + P2.W * (D ^ P1)) / DD;
		D.Normalize();
		return 1;
	}
}


UE_DECLARE_LWC_TYPE(Plane,4);
template<> struct TIsPODType<FPlane4f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FPlane4f> { enum { Value = true }; };

template<> struct TIsPODType<FPlane4d> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FPlane4d> { enum { Value = true }; };


#ifdef _MSC_VER
#pragma warning (pop)
#endif
