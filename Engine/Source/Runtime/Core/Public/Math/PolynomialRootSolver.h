// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/UnrealMathUtility.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"

namespace UE::Math
{

/**
 * Find roots of a polynomial of a specified degree
 */
template<typename RealType, int32 PolynomialDegree>
struct TPolynomialRootSolver
{
	static_assert(PolynomialDegree >= 2, "PolynomialDegree must be 2 or higher");

	// Holds last-found polynomial roots
	TArray<RealType, TInlineAllocator<PolynomialDegree>> Roots;

	TPolynomialRootSolver() = default;

	/**
	 * Find roots within the specified open interval (RangeStart, RangeEnd)  (i.e. roots at either extreme are not returned)
	 *
	 * @param PolyCoeffs The coefficients of the polynomial such that PolyCoeffs[i] is the coefficient of the x^i term. Must have at least PolynomialDegree + 1 elements.
	 * @param RangeStart Start of the open range to search for roots
	 * @param RangeEnd End of the open range to search for roots
	 * @param Tolerance Absolute tolerance for the returned root
	 * @param MaxNewtonIterations Maximum number of newton/bisection iterations to perform internally when finding a root
	 * @param NearRootTolerance Tolerance for finding almost-roots, i.e. cases where the polynomial just grazes 0 without crossing
	 */
	TPolynomialRootSolver(TArrayView<const RealType> PolyCoeffs, RealType RangeStart, RealType RangeEnd, 
		RealType Tolerance = (RealType)UE_SMALL_NUMBER, int32 MaxNewtonIterations = 20, RealType NearRootTolerance = (RealType)UE_SMALL_NUMBER)
	{
		FindRootsInRange(PolyCoeffs, RangeStart, RangeEnd, Tolerance, MaxNewtonIterations, NearRootTolerance);
	}

	/**
	 * Find roots within the specified open interval (RangeStart, RangeEnd)  (i.e. roots at either extreme are not returned)
	 * 
	 * @param PolyCoeffs The coefficients of the polynomial such that PolyCoeffs[i] is the coefficient of the x^i term. Must have at least PolynomialDegree + 1 elements.
	 * @param RangeStart Start of the open range to search for roots
	 * @param RangeEnd End of the open range to search for roots
	 * @param Tolerance Absolute tolerance for the returned roots
	 * @param MaxNewtonIterations Maximum number of newton/bisection iterations to perform internally when finding a root
	 * @param NearRootTolerance Tolerance for finding almost-roots, i.e. cases where the polynomial just grazes 0 without crossing
	 * @return Number of roots found
	 */
	int32 FindRootsInRange(TArrayView<const RealType> PolyCoeffs, RealType RangeStart, RealType RangeEnd, 
		RealType Tolerance = (RealType)UE_SMALL_NUMBER, int32 MaxNewtonIterations = 20, RealType NearRootTolerance = (RealType)UE_SMALL_NUMBER)
	{
		Roots.Reset();

		if (!ensure(PolyCoeffs.Num() >= PolynomialDegree + 1))
		{
			return 0;
		}

		// Helper to evaluate a polynomial
		auto EvalPolynomial = [](RealType* Coeffs, int32 Degree, RealType Param)
		{
			RealType Value = Coeffs[Degree];
			for (int32 CoeffIdx = Degree - 1; CoeffIdx >= 0; --CoeffIdx)
			{
				Value = Value * Param + Coeffs[CoeffIdx];
			}
			return Value;
		};

		// Local arrays to store polynomial coefficients and coefficients of derivatives
		RealType LocalCoeffs[PolynomialDegree + 1];
		RealType DerivCoeffs[PolynomialDegree];

		// Local storage for found roots (including incrementally of derivatives)
		// Note: has an extra element so we can add RangeEnd to end, to make iteration over test intervals easier below
		RealType FoundRoots[PolynomialDegree + 1];
		int32 NumFoundRoots = 0;

		// Build quadratic derivative, rescaled so that the constant coefficient is the same as the input polynomial (i.e., divided by Factorial(PolynomialDegree - 2))
		LocalCoeffs[0] = PolyCoeffs[PolynomialDegree - 2];
		LocalCoeffs[1] = RealType(PolynomialDegree - 1) * PolyCoeffs[PolynomialDegree - 1];
		LocalCoeffs[2] = ((RealType).5 * RealType(PolynomialDegree * (PolynomialDegree - 1))) * PolyCoeffs[PolynomialDegree];
		// Directly solve quadratic roots
		RealType Discrim = LocalCoeffs[1] * LocalCoeffs[1] - (RealType)4.0 * LocalCoeffs[0] * LocalCoeffs[2];
		if (Discrim >= 0)
		{
			RealType RootDiscrim = FMath::Sqrt(Discrim);
			RealType BPlusSignBTimesRootDiscrim = -RealType(.5) * (LocalCoeffs[1] + (LocalCoeffs[1] < 0 ? -RootDiscrim : RootDiscrim));
			// Guard against divide by exact 0 to avoid fast math failure case; otherwise rely on the (RangeStart, RangeEnd) interval test to filter bad results
			if (BPlusSignBTimesRootDiscrim != 0)
			{
				RealType Root0 = LocalCoeffs[0] / BPlusSignBTimesRootDiscrim;
				if (Root0 > RangeStart && Root0 < RangeEnd)
				{
					FoundRoots[0] = Root0;
					NumFoundRoots++;
				}
			}
			if (LocalCoeffs[2] != 0)
			{
				RealType Root1 = BPlusSignBTimesRootDiscrim / LocalCoeffs[2];
				if (Root1 > RangeStart && Root1 < RangeEnd)
				{
					FoundRoots[NumFoundRoots++] = Root1;
					// Order roots and filter exact duplicates
					if (NumFoundRoots == 2)
					{
						if (FoundRoots[1] < FoundRoots[0])
						{
							Swap(FoundRoots[0], FoundRoots[1]);
						}
						else if (FoundRoots[0] == FoundRoots[1])
						{
							NumFoundRoots = 1;
						}
					}
				}
			}
		}

		// Note: this constexpr if statement needed to work around compiler warning that the below loop won't execute for quadratic case ("Ill-defined for-loop.  Loop body not executed.")
		if constexpr (PolynomialDegree >= 3)
		{

			for (int32 CurDegree = 3; CurDegree <= PolynomialDegree; ++CurDegree)
			{
				// Add a fake root at the range end to facilitate iteration below
				FoundRoots[NumFoundRoots] = RangeEnd;

				// Use last LocalCoeffs as the derivative of the next one
				// Multiply back scale factor s.t. the constant factor can always be directly copied from the source polynomial
				RealType DerivScale = RealType(1 + PolynomialDegree - CurDegree);
				for (int32 CoeffIdx = 0; CoeffIdx < CurDegree; ++CoeffIdx)
				{
					DerivCoeffs[CoeffIdx] = DerivScale * LocalCoeffs[CoeffIdx];
				}

				// Integrate derivative to get next polynomial
				if (CurDegree < PolynomialDegree)
				{
					for (int32 CoeffIdx = CurDegree; CoeffIdx > 0; --CoeffIdx)
					{
						LocalCoeffs[CoeffIdx] = DerivCoeffs[CoeffIdx - 1] / (RealType)CoeffIdx;
					}
					// Copy constant coefficient from the source polynomial
					LocalCoeffs[0] = PolyCoeffs[PolynomialDegree - CurDegree];
				}
				// Once we're back to the original polynomial, we can just copy its coefficients directly instead
				else
				{
					for (int32 CoeffIdx = 0; CoeffIdx < PolynomialDegree + 1; ++CoeffIdx)
					{
						LocalCoeffs[CoeffIdx] = PolyCoeffs[CoeffIdx];
					}
				}

				// Check for roots in each interval from (range start, deriv root 0, ..., deriv root N, range end)
				RealType CurStart = RangeStart;
				RealType CurStartValue = EvalPolynomial(LocalCoeffs, CurDegree, CurStart);
				checkSlow(NumFoundRoots < PolynomialDegree + 1);
				int32 NumNewRoots = 0;
				for (int32 RootIdx = 0; RootIdx < NumFoundRoots + 1; ++RootIdx)
				{
					RealType CurEnd = FoundRoots[RootIdx];

					// expect roots to be in increasing order
					checkSlow(CurStart < CurEnd);

					RealType CurEndValue = EvalPolynomial(LocalCoeffs, CurDegree, CurEnd);

					// If sign changes, find root inside interval
					if (CurStartValue * CurEndValue < 0)
					{
						RealType SearchBegin = CurStart, SearchBeginValue = CurStartValue;
						RealType BeginSign = FMath::Sign(SearchBeginValue);
						RealType SearchEnd = CurEnd;
						RealType SearchParam = (RealType).5 * (SearchBegin + SearchEnd);
						int32 SearchIters = MaxNewtonIterations;
						RealType SearchStepSize;
						do
						{
							RealType SearchParamValue = EvalPolynomial(LocalCoeffs, CurDegree, SearchParam);
							// Reduce the search range based on the sign of the search value
							if (SearchParamValue * BeginSign > 0)
							{
								SearchBegin = SearchParam;
							}
							else
							{
								SearchEnd = SearchParam;
							}
							RealType SearchParamDerivValue = EvalPolynomial(DerivCoeffs, CurDegree - 1, SearchParam);
							RealType NextSearchParam;
							// Guard against divide by exact 0 to avoid fast math failure case; otherwise rely on the (SearchBegin, SearchEnd) interval test to filter bad results
							if (SearchParamDerivValue != 0)
							{
								RealType NewtonApproxRoot = SearchParam - SearchParamValue / SearchParamDerivValue;
								if (NewtonApproxRoot > SearchBegin && NewtonApproxRoot < SearchEnd)
								{
									NextSearchParam = NewtonApproxRoot;
								}
								else
								{
									NextSearchParam = (RealType).5 * (SearchBegin + SearchEnd);
								}
							}
							else
							{
								NextSearchParam = (RealType).5 * (SearchBegin + SearchEnd);
							}
							SearchStepSize = FMath::Abs(NextSearchParam - SearchParam);
							SearchParam = NextSearchParam;
						} while (SearchStepSize > Tolerance && --SearchIters > 0);

						// Write the new roots back to the FoundRoots array (since this will always trail behind the index we read from, above)
						FoundRoots[NumNewRoots++] = SearchParam;
					}
					// No sign change; check for a near-root value at interval boundary
					else if (RootIdx > 0 && CurStartValue <= NearRootTolerance)
					{
						FoundRoots[NumNewRoots++] = CurStart;
					}

					CurStart = CurEnd;
					CurStartValue = CurEndValue;
				}
				NumFoundRoots = NumNewRoots;
			}

		}

		// Copy out the final roots
		for (int32 Idx = 0; Idx < NumFoundRoots; ++Idx)
		{
			Roots.Add(FoundRoots[Idx]);
		}
		return NumFoundRoots;
	}
};

} // namespace UE::Math

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Containers/StaticArray.h"
#endif
