// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "Math/MathFwd.h" // IWYU pragma: export
#include "Math/Vector.h"
#include "Math/VectorRegister.h"
#include "Math/Rotator.h"
#include "Math/Matrix.h"
#include "Misc/LargeWorldCoordinatesSerializer.h"
#include "UObject/ObjectVersion.h"

class Error;

namespace UE
{
namespace Math
{

/**
 * Floating point quaternion that can represent a rotation about an axis in 3-D space.
 * The X, Y, Z, W components also double as the Axis/Angle format.
 *
 * Order matters when composing quaternions: C = A * B will yield a quaternion C that logically
 * first applies B then A to any subsequent transformation (right first, then left).
 * Note that this is the opposite order of FTransform multiplication.
 *
 * Example: LocalToWorld = (LocalToWorld * DeltaRotation) will change rotation in local space by DeltaRotation.
 * Example: LocalToWorld = (DeltaRotation * LocalToWorld) will change rotation in world space by DeltaRotation.
 */

template<typename T>
struct alignas(16) TQuat
{
	// Can't have a UE_REQUIRES in the declaration because of the forward declarations, so check for allowed types here.
	static_assert(std::is_floating_point_v<T>, "TQuat only supports float and double types.");

public:
	/** Type of the template param (float or double) */
	using FReal = T;
	using QuatVectorRegister = TVectorRegisterType<T>;

	/** The quaternion's X-component. */
	T X;

	/** The quaternion's Y-component. */
	T Y;

	/** The quaternion's Z-component. */
	T Z;

	/** The quaternion's W-component. */
	T W;

public:

	/** Identity quaternion. */
	CORE_API static const TQuat<T> Identity;

public:

	/** Default constructor (no initialization). */
	[[nodiscard]] TQuat() = default;

	/**
	 * Creates and initializes a new quaternion, with the W component either 0 or 1.
	 *
	 * @param EForceInit Force init enum: if equal to ForceInitToZero then W is 0, otherwise W = 1 (creating an identity transform)
	 */
	[[nodiscard]] explicit UE_FORCEINLINE_HINT TQuat(EForceInit);

	/**
	 * Constructor.
	 *
	 * @param InX X component of the quaternion
	 * @param InY Y component of the quaternion
	 * @param InZ Z component of the quaternion
	 * @param InW W component of the quaternion
	 */
	[[nodiscard]] inline TQuat(T InX, T InY, T InZ, T InW);

	/**
	 * Initializes all elements to V
	 */
	template<typename DummyT = T UE_REQUIRES(std::is_arithmetic_v<T>)>
	[[nodiscard]] explicit inline TQuat(T V)
		: X(V)
		, Y(V)
		, Z(V)
		, W(V)
	{
		DiagnosticCheckNaN();
	}

protected:
	/**
	 * Creates and initializes a new quaternion from the XYZW values in the given VectorRegister4Float.
	 *
	 * @param V XYZW components of the quaternion packed into a single VectorRegister4Float
	 */
	[[nodiscard]] explicit TQuat(QuatVectorRegister V);

public:

	[[nodiscard]] UE_FORCEINLINE_HINT static TQuat<T> MakeFromVectorRegister(QuatVectorRegister V)
	{
		return TQuat<T>(V);
	}

	/**
	 * Creates and initializes a new quaternion from the given rotator.
	 *
	 * @param R The rotator to initialize from.
	 */
	explicit TQuat(const TRotator<T>& R);

	[[nodiscard]] UE_FORCEINLINE_HINT static TQuat<T> MakeFromRotator(const TRotator<T>& R)
	{
		return TQuat<T>(R);
	}

	/**
	 * Creates and initializes a new quaternion from the given matrix.
	 *
	 * @param M The rotation matrix to initialize from.
	 */
	explicit TQuat(const TMatrix<T>& M);

	/**
	 * Creates and initializes a new quaternion from the a rotation around the given axis.
	 *
	 * @param Axis assumed to be a normalized vector
	 * @param Angle angle to rotate above the given axis (in radians)
	 */
	TQuat(TVector<T> Axis, T AngleRad);

public:

	/**
	 * Gets the result of adding a Quaternion to this.
	 * This is a component-wise addition; composing quaternions should be done via multiplication.
	 *
	 * @param Q The Quaternion to add.
	 * @return The result of addition.
	 */
	[[nodiscard]] inline TQuat<T> operator+(const TQuat<T>& Q) const;

	/**
	 * Adds to this quaternion.
	 * This is a component-wise addition; composing quaternions should be done via multiplication.
	 *
	 * @param Other The quaternion to add to this.
	 * @return Result after addition.
	 */
	inline TQuat<T> operator+=(const TQuat<T>& Q);

	/**
	 * Gets the result of subtracting a Quaternion to this.
	 * This is a component-wise subtraction; composing quaternions should be done via multiplication.
	 *
	 * @param Q The Quaternion to subtract.
	 * @return The result of subtraction.
	 */
	[[nodiscard]] inline TQuat<T> operator-(const TQuat<T>& Q) const;

	/**
	* Negates the quaternion. Note that this represents the same rotation.
	*
	* @return The result of negation.
	*/
	[[nodiscard]] inline TQuat<T> operator-() const;

	/**
	 * Checks whether another Quaternion is equal to this, within specified tolerance.
	 *
	 * @param Q The other Quaternion.
	 * @param Tolerance Error tolerance for comparison with other Quaternion.
	 * @return true if two Quaternions are equal, within specified tolerance, otherwise false.
	 */
	[[nodiscard]] inline bool Equals(const TQuat<T>& Q, T Tolerance=UE_KINDA_SMALL_NUMBER) const;

	/**
	 * Checks whether this Quaternion is an Identity Quaternion.
	 * Assumes Quaternion tested is normalized.
	 *
	 * @param Tolerance Error tolerance for comparison with Identity Quaternion.
	 * @return true if Quaternion is a normalized Identity Quaternion.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsIdentity(T Tolerance=UE_SMALL_NUMBER) const;

	/**
	 * Subtracts another quaternion from this.
	 * This is a component-wise subtraction; composing quaternions should be done via multiplication.
	 *
	 * @param Q The other quaternion.
	 * @return reference to this after subtraction.
	 */
	inline TQuat<T> operator-=(const TQuat<T>& Q);

	/**
	 * Gets the result of multiplying this by another quaternion (this * Q).
	 *
	 * Order matters when composing quaternions: C = A * B will yield a quaternion C that logically
	 * first applies B then A to any subsequent transformation (right first, then left).
	 *
	 * @param Q The Quaternion to multiply this by.
	 * @return The result of multiplication (this * Q).
	 */
	[[nodiscard]] inline TQuat<T> operator*(const TQuat<T>& Q) const;

	/**
	 * Multiply this by a quaternion (this = this * Q).
	 *
	 * Order matters when composing quaternions: C = A * B will yield a quaternion C that logically
	 * first applies B then A to any subsequent transformation (right first, then left).
	 *
	 * @param Q the quaternion to multiply this by.
	 * @return The result of multiplication (this * Q).
	 */
	inline TQuat<T> operator*=(const TQuat<T>& Q);

	/**
	 * Rotate a vector by this quaternion.
	 *
	 * @param V the vector to be rotated
	 * @return vector after rotation
	 * @see RotateVector
	 */
	[[nodiscard]] TVector<T> operator*(const TVector<T>& V) const;

	/** 
	 * Multiply this by a matrix.
	 * This matrix conversion came from
	 * http://www.m-hikari.com/ija/ija-password-2008/ija-password17-20-2008/aristidouIJA17-20-2008.pdf
	 * used for non-uniform scaling transform.
	 *
	 * @param M Matrix to multiply by.
	 * @return Matrix result after multiplication.
	 */	
	[[nodiscard]] CORE_API TMatrix<T> operator*(const TMatrix<T>& M) const;
	
	/**
	 * Multiply this quaternion by a scaling factor.
	 *
	 * @param Scale The scaling factor.
	 * @return a reference to this after scaling.
	 */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	inline TQuat<T> operator*=(const FArg Scale)
	{
#if PLATFORM_ENABLE_VECTORINTRINSICS
		QuatVectorRegister A = VectorLoadAligned(this);
		QuatVectorRegister B = VectorSetFloat1(Scale);
		VectorStoreAligned(VectorMultiply(A, B), this);
#else
		X *= Scale;
		Y *= Scale;
		Z *= Scale;
		W *= Scale;
#endif // PLATFORM_ENABLE_VECTORINTRINSICS

		DiagnosticCheckNaN();
		return *this;
	}

	/**
	 * Get the result of scaling this quaternion.
	 *
	 * @param Scale The scaling factor.
	 * @return The result of scaling.
	 */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	[[nodiscard]] inline TQuat<T> operator*(const FArg Scale) const
	{
#if PLATFORM_ENABLE_VECTORINTRINSICS
		QuatVectorRegister A = VectorLoadAligned(this);
		QuatVectorRegister B = VectorSetFloat1((T)Scale);
		return TQuat(VectorMultiply(A, B));
#else
		return TQuat(Scale * X, Scale * Y, Scale * Z, Scale * W);
#endif // PLATFORM_ENABLE_VECTORINTRINSICS
	}
	
	/**
	 * Divide this quaternion by scale.
	 *
	 * @param Scale What to divide by.
	 * @return a reference to this after scaling.
	 */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	inline TQuat<T> operator/=(const FArg Scale)
	{
#if PLATFORM_ENABLE_VECTORINTRINSICS
		QuatVectorRegister A = VectorLoadAligned(this);
		QuatVectorRegister B = VectorSetFloat1((T)Scale);
		VectorStoreAligned(VectorDivide(A, B), this);
#else
		const T Recip = T(1.0f) / Scale;
		X *= Recip;
		Y *= Recip;
		Z *= Recip;
		W *= Recip;
#endif // PLATFORM_ENABLE_VECTORINTRINSICS

		DiagnosticCheckNaN();
		return *this;
	}

	/**
	 * Divide this quaternion by scale.
	 *
	 * @param Scale What to divide by.
	 * @return new Quaternion of this after division by scale.
	 */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	[[nodiscard]] inline TQuat<T> operator/(const FArg Scale) const
	{
#if PLATFORM_ENABLE_VECTORINTRINSICS
		QuatVectorRegister A = VectorLoadAligned(this);
		QuatVectorRegister B = VectorSetFloat1(Scale);
		return TQuat(VectorDivide(A, B));
#else
		const T Recip = 1.0f / Scale;
		return TQuat(X * Recip, Y * Recip, Z * Recip, W * Recip);
#endif // PLATFORM_ENABLE_VECTORINTRINSICS
	}

	/**
	 * Identical implementation for TQuat properties. 
	 * Avoids intrinsics to remain consistent with previous per-property comparison.
	 */
	[[nodiscard]] bool Identical(const TQuat* Q, const uint32 PortFlags) const;

 	/**
	 * Checks whether two quaternions are identical.
	 * This is an exact comparison per-component;see Equals() for a comparison
	 * that allows for a small error tolerance and flipped axes of rotation.
	 *
	 * @param Q The other quaternion.
	 * @return true if two quaternion are identical, otherwise false.
	 * @see Equals
	 */
	[[nodiscard]] bool operator==(const TQuat<T>& Q) const;

 	/**
	 * Checks whether two quaternions are not identical.
	 *
	 * @param Q The other quaternion.
	 * @return true if two quaternion are not identical, otherwise false.
	 */
	[[nodiscard]] bool operator!=(const TQuat<T>& Q) const;

	/**
	 * Calculates dot product of two quaternions.
	 *
	 * @param Q The other quaternions.
	 * @return The dot product.
	 */
	[[nodiscard]] T operator|(const TQuat<T>& Q) const;

public:

	/**
	 * Convert a vector of floating-point Euler angles (in degrees) into a Quaternion.
	 * 
	 * @param Euler the Euler angles
	 * @return constructed TQuat
	 */
	static CORE_API TQuat<T> MakeFromEuler(const TVector<T>& Euler);

	/** Convert a Quaternion into floating-point Euler angles (in degrees). */
	[[nodiscard]] CORE_API TVector<T> Euler() const;

	/**
	 * Normalize this quaternion if it is large enough.
	 * If it is too small, returns an identity quaternion.
	 *
	 * @param Tolerance Minimum squared length of quaternion for normalization.
	 */
	inline void Normalize(T Tolerance = UE_SMALL_NUMBER);

	/**
	 * Get a normalized copy of this quaternion.
	 * If it is too small, returns an identity quaternion.
	 *
	 * @param Tolerance Minimum squared length of quaternion for normalization.
	 */
	[[nodiscard]] inline TQuat<T> GetNormalized(T Tolerance = UE_SMALL_NUMBER) const;

	// Return true if this quaternion is normalized
	[[nodiscard]] bool IsNormalized() const;

	/**
	 * Get the length of this quaternion.
	 *
	 * @return The length of this quaternion.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T Size() const;

	/**
	 * Get the length squared of this quaternion.
	 *
	 * @return The length of this quaternion.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T SizeSquared() const;

	/** 
	 * Get the angle in radians of this quaternion 
	 * @warning : The rotation returned may not be the shortest version.
	 *            You may wish to call GetShortestArcWith(FQuat::Identity) first.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T GetAngle() const;

	/** 
	 * get the axis and angle of rotation of this quaternion
	 *
	 * @param Axis{out] Normalized rotation axis of the quaternion
	 * @param Angle{out] Angle of the quaternion in radians
	 * @warning : Requires this quaternion to be normalized.
	 * @warning : The rotation returned may not be the shortest version.
	 *            You may wish to call GetShortestArcWith(FQuat::Identity) first.
	 */
	void ToAxisAndAngle(TVector<T>& Axis, float& Angle) const;
	void ToAxisAndAngle(TVector<T>& Axis, double& Angle) const;

	/**
	 * Get the rotation vector corresponding to this quaternion. 
	 * The direction of the vector represents the rotation axis, 
	 * and the magnitude the angle in radians.
	 * @warning : Requires this quaternion to be normalized. 
	 * @warning : The rotation returned may not be the shortest version. 
	 *            You may wish to call GetShortestArcWith(FQuat::Identity) first.
	 */
	[[nodiscard]] TVector<T> ToRotationVector() const;

	/**
	 * Constructs a quaternion corresponding to the rotation vector. 
	 * The direction of the vector represents the rotation axis, 
	 * and the magnitude the angle in radians.
	 */
	[[nodiscard]] static TQuat<T> MakeFromRotationVector(const TVector<T>& RotationVector);

	/** 
	 * Get the swing and twist decomposition for a specified axis
	 *
	 * @param InTwistAxis Axis to use for decomposition
	 * @param OutSwing swing component quaternion
	 * @param OutTwist Twist component quaternion
	 * @warning assumes normalized quaternion and twist axis
	 */
	CORE_API void ToSwingTwist(const TVector<T>& InTwistAxis, TQuat<T>& OutSwing, TQuat<T>& OutTwist) const;

	/**
	 * Get the twist angle (in radians) for a specified axis
	 *
	 * @param TwistAxis Axis to use for decomposition
	 * @return Twist angle (in radians)
	 * @warning assumes normalized quaternion and twist axis
	 */
	[[nodiscard]] CORE_API T GetTwistAngle(const TVector<T>& TwistAxis) const;

	/**
	 * Rotate a vector by this quaternion.
	 *
	 * @param V the vector to be rotated
	 * @return vector after rotation
	 */
	[[nodiscard]] TVector<T> RotateVector(TVector<T> V) const;
	
	/**
	 * Rotate a vector by the inverse of this quaternion.
	 *
	 * @param V the vector to be rotated
	 * @return vector after rotation by the inverse of this quaternion.
	 */
	[[nodiscard]] TVector<T> UnrotateVector(TVector<T> V) const;

	/**
	 * @return quaternion with W=0 and V=theta*v.
	 * @warning : The rotation returned may not be the shortest version.
	 *            You may wish to call GetShortestArcWith(FQuat::Identity) first.
	 */
	[[nodiscard]] CORE_API TQuat<T> Log() const;

	/**
	 * @note Exp should really only be used after Log.
	 * Assumes a quaternion with W=0 and V=theta*v (where |v| = 1).
	 * Exp(q) = (sin(theta)*v, cos(theta))
	 */
	[[nodiscard]] CORE_API TQuat<T> Exp() const;

	/**
	 * @return inverse of this quaternion
	 * @warning : Requires this quaternion to be normalized.
	 */
	[[nodiscard]] TQuat<T> Inverse() const;

	/**
	 * Enforce that the delta between this Quaternion and another one represents
	 * the shortest possible rotation angle
	 */
	void EnforceShortestArcWith(const TQuat<T>& OtherQuat);

	/**
	 * Returns the version of this quaternion that represents the shortest rotation angle 
	 * to the other quaternion
	 */
	[[nodiscard]] TQuat<T> GetShortestArcWith(const TQuat<T>& OtherQuat) const;

	/** Get the forward direction (X axis) after it has been rotated by this Quaternion. */
	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> GetAxisX() const;

	/** Get the right direction (Y axis) after it has been rotated by this Quaternion. */
	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> GetAxisY() const;

	/** Get the up direction (Z axis) after it has been rotated by this Quaternion. */
	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> GetAxisZ() const;

	/** Get the forward direction (X axis) after it has been rotated by this Quaternion. */
	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> GetForwardVector() const;

	/** Get the right direction (Y axis) after it has been rotated by this Quaternion. */
	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> GetRightVector() const;

	/** Get the up direction (Z axis) after it has been rotated by this Quaternion. */
	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> GetUpVector() const;

	/** Convert a rotation into a unit vector facing in its direction. Equivalent to GetForwardVector(). */
	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> Vector() const;

	/** Get the TRotator<T> representation of this Quaternion. */
	[[nodiscard]] CORE_API TRotator<T> Rotator() const;

	/** Get the TMatrix<T> representation of this Quaternion. */
	[[nodiscard]] UE_FORCEINLINE_HINT TMatrix<T> ToMatrix() const;

	/** Get the TMatrix<T> representation of this Quaternion and store it in Mat */
	CORE_API void ToMatrix(TMatrix<T>& Mat) const;

	/**
	 * Get the axis of rotation of the Quaternion.
	 * This is the axis around which rotation occurs to transform the canonical coordinate system to the target orientation.
	 * For the identity Quaternion which has no such rotation, TVector<T>(1,0,0) is returned.
	 */
	[[nodiscard]] inline TVector<T> GetRotationAxis() const;

	/** Find the angular distance between two rotation quaternions (in radians) */
	[[nodiscard]] T AngularDistance(const TQuat<T>& Q) const;

	/**
	 * Serializes the vector compressed for e.g. network transmission.
	 * @param Ar Archive to serialize to/ from.
	 * @return false to allow the ordinary struct code to run (this never happens).
	 */
	CORE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/**
	 * Utility to check if there are any non-finite values (NaN or Inf) in this Quat.
	 *
	 * @return true if there are any non-finite values in this Quaternion, otherwise false.
	 */
	[[nodiscard]] bool ContainsNaN() const;

	/**
	 * Get a textual representation of the vector.
	 *
	 * @return Text describing the vector.
	 */
	[[nodiscard]] FString ToString() const;

	/**
	 * Initialize this TQuat from a FString. 
	 * The string is expected to contain X=, Y=, Z=, W=, otherwise 
	 * this TQuat will have indeterminate (invalid) values.
	 *
	 * @param InSourceString FString containing the quaternion values.
	 * @return true if the TQuat was initialized; false otherwise.
	 */
	bool InitFromString(const FString& InSourceString);

public:

#if ENABLE_NAN_DIAGNOSTIC
	inline void DiagnosticCheckNaN() const
	{
		if (ContainsNaN())
		{
			logOrEnsureNanError(TEXT("Quat contains NaN: %s"), *ToString());
		}
	}

	inline void DiagnosticCheckNaN(const TCHAR* Message) const
	{
		if (ContainsNaN())
		{
			logOrEnsureNanError(TEXT("%s: Quat contains NaN: %s"), Message, *ToString());
		}
	}
#else
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN() const {}
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN(const TCHAR* Message) const {}
#endif

public:

	/**
	 * Generates the 'smallest' (geodesic) rotation between two vectors of arbitrary length.
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT TQuat<T> FindBetween(const TVector<T>& Vector1, const TVector<T>& Vector2)
	{
		return FindBetweenVectors(Vector1, Vector2);
	}

	/**
	 * Generates the 'smallest' (geodesic) rotation between two normals (assumed to be unit length).
	 */
	[[nodiscard]] static CORE_API TQuat<T> FindBetweenNormals(const TVector<T>& Normal1, const TVector<T>& Normal2);

	/**
	 * Generates the 'smallest' (geodesic) rotation between two vectors of arbitrary length.
	 */
	[[nodiscard]] static CORE_API TQuat<T> FindBetweenVectors(const TVector<T>& Vector1, const TVector<T>& Vector2);

	/**
	 * Error measure (angle) between two quaternions, ranged [0..1].
	 * Returns the hypersphere-angle between two quaternions; alignment shouldn't matter, though 
	 * @note normalized input is expected.
	 */
	[[nodiscard]] static inline T Error(const TQuat<T>& Q1, const TQuat<T>& Q2);

	/**
	 * TQuat<T>::Error with auto-normalization.
	 */
	[[nodiscard]] static inline T ErrorAutoNormalize(const TQuat<T>& A, const TQuat<T>& B);

	/** 
	 * Fast Linear Quaternion Interpolation.
	 * Result is NOT normalized.
	 */
	[[nodiscard]] static inline TQuat<T> FastLerp(const TQuat<T>& A, const TQuat<T>& B, const T Alpha);

	/** 
	 * Bi-Linear Quaternion interpolation.
	 * Result is NOT normalized.
	 */
	[[nodiscard]] static inline TQuat<T> FastBilerp(const TQuat<T>& P00, const TQuat<T>& P10, const TQuat<T>& P01, const TQuat<T>& P11, T FracX, T FracY);


	/** Spherical interpolation. Will correct alignment. Result is NOT normalized. */
	[[nodiscard]] static CORE_API TQuat<T> Slerp_NotNormalized(const TQuat<T>& Quat1, const TQuat<T>& Quat2, T Slerp);

	/** Spherical interpolation. Will correct alignment. Result is normalized. */
	[[nodiscard]] static UE_FORCEINLINE_HINT TQuat<T> Slerp(const TQuat<T>& Quat1, const TQuat<T>& Quat2, T Slerp)
	{
		return Slerp_NotNormalized(Quat1, Quat2, Slerp).GetNormalized();
	}

	/**
	 * Simpler Slerp that doesn't do any checks for 'shortest distance' etc.
	 * We need this for the cubic interpolation stuff so that the multiple Slerps dont go in different directions.
	 * Result is NOT normalized.
	 */
	[[nodiscard]] static CORE_API TQuat<T> SlerpFullPath_NotNormalized(const TQuat<T>& quat1, const TQuat<T>& quat2, T Alpha);

	/**
	 * Simpler Slerp that doesn't do any checks for 'shortest distance' etc.
	 * We need this for the cubic interpolation stuff so that the multiple Slerps dont go in different directions.
	 * Result is normalized.
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT TQuat<T> SlerpFullPath(const TQuat<T>& quat1, const TQuat<T>& quat2, T Alpha)
	{
		return SlerpFullPath_NotNormalized(quat1, quat2, Alpha).GetNormalized();
	}
	
	/**
	 * Given start and end quaternions of quat1 and quat2, and tangents at those points tang1 and tang2, calculate the point at Alpha (between 0 and 1) between them. Result is normalized.
	 * This will correct alignment by ensuring that the shortest path is taken.
	 */
	[[nodiscard]] static CORE_API TQuat<T> Squad(const TQuat<T>& quat1, const TQuat<T>& tang1, const TQuat<T>& quat2, const TQuat<T>& tang2, T Alpha);

	/**
	 * Simpler Squad that doesn't do any checks for 'shortest distance' etc.
	 * Given start and end quaternions of quat1 and quat2, and tangents at those points tang1 and tang2, calculate the point at Alpha (between 0 and 1) between them. Result is normalized.
	 */
	[[nodiscard]] static CORE_API TQuat<T> SquadFullPath(const TQuat<T>& quat1, const TQuat<T>& tang1, const TQuat<T>& quat2, const TQuat<T>& tang2, T Alpha);

	/**
	 * Calculate tangents between given points
	 *
	 * @param PrevP quaternion at P-1
	 * @param P quaternion to return the tangent
	 * @param NextP quaternion P+1
	 * @param Tension @todo document
	 * @param OutTan Out control point
	 */
	static CORE_API void CalcTangents(const TQuat<T>& PrevP, const TQuat<T>& P, const TQuat<T>& NextP, T Tension, TQuat<T>& OutTan);

	[[nodiscard]] TTuple<T, T, T> ToLUFEuler() const;

	// Converts to euler angles (in rads) in verse conventions
	[[nodiscard]] static TQuat MakeFromLUFEuler(const TTuple<T, T, T>& InLUFEuler);

public:

	bool Serialize(FArchive& Ar)
	{
		Ar << (TQuat<T>&)*this;
		return true;
	}

	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		if constexpr (std::is_same_v<T, float>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Quat, Quat4f, Quat4d);
		}
		else
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Quat, Quat4d, Quat4f);
		}
	}

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TQuat(const TQuat<FArg>& From)
		: TQuat<T>((T)From.X, (T)From.Y, (T)From.Z, (T)From.W)
	{
	}
};

/**
 * Serializes the quaternion.
 *
 * @param Ar Reference to the serialization archive.
 * @param F Reference to the quaternion being serialized.
 * @return Reference to the Archive after serialization.
 */
inline FArchive& operator<<(FArchive& Ar, TQuat<float>& F)
{
	return Ar << F.X << F.Y << F.Z << F.W;
}

inline FArchive& operator<<(FArchive& Ar, TQuat<double>& F)
{
	if (Ar.UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
	{
		return Ar << F.X << F.Y << F.Z << F.W;
	}
	else
	{
		checkf(Ar.IsLoading(), TEXT("float -> double conversion applied outside of load!"));
		// Stored as floats, so serialize float and copy.
		float X, Y, Z, W;
		Ar << X << Y << Z << W;
		if(Ar.IsLoading())
		{
			F = TQuat<double>(X, Y, Z, W);
		}
	}
	return Ar;
}

#if !defined(_MSC_VER) || defined(__clang__)  // MSVC can't forward declare explicit specializations
template<> CORE_API const FQuat4f FQuat4f::Identity;
template<> CORE_API const FQuat4d FQuat4d::Identity;
#endif
/* TQuat inline functions
 *****************************************************************************/
template<typename T>
inline TQuat<T>::TQuat(const UE::Math::TMatrix<T>& M)
{
	// If Matrix is NULL, return Identity quaternion. If any of them is 0, you won't be able to construct rotation
	// if you have two plane at least, we can reconstruct the frame using cross product, but that's a bit expensive op to do here
	// for now, if you convert to matrix from 0 scale and convert back, you'll lose rotation. Don't do that. 
	if (M.GetScaledAxis(EAxis::X).IsNearlyZero() || M.GetScaledAxis(EAxis::Y).IsNearlyZero() || M.GetScaledAxis(EAxis::Z).IsNearlyZero())
	{
		*this = TQuat<T>::Identity;
		return;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Make sure the Rotation part of the Matrix is unit length.
	// Changed to this (same as RemoveScaling) from RotDeterminant as using two different ways of checking unit length matrix caused inconsistency. 
	if (!ensure((FMath::Abs(1.f - M.GetScaledAxis(EAxis::X).SizeSquared()) <= UE_KINDA_SMALL_NUMBER) && (FMath::Abs(1.f - M.GetScaledAxis(EAxis::Y).SizeSquared()) <= UE_KINDA_SMALL_NUMBER) && (FMath::Abs(1.f - M.GetScaledAxis(EAxis::Z).SizeSquared()) <= UE_KINDA_SMALL_NUMBER)))
	{
		*this = TQuat<T>::Identity;
		return;
	}
#endif

	//const MeReal *const t = (MeReal *) tm;
	T s;

	// Check diagonal (trace)
	const T tr = M.M[0][0] + M.M[1][1] + M.M[2][2];

	if (tr > 0.0f) 
	{
		T InvS = FMath::InvSqrt(tr + T(1.f));
		this->W = T(T(0.5f) * (T(1.f) / InvS));
		s = T(0.5f) * InvS;

		this->X = ((M.M[1][2] - M.M[2][1]) * s);
		this->Y = ((M.M[2][0] - M.M[0][2]) * s);
		this->Z = ((M.M[0][1] - M.M[1][0]) * s);
	} 
	else 
	{
		// diagonal is negative
		int32 i = 0;

		if (M.M[1][1] > M.M[0][0])
			i = 1;

		if (M.M[2][2] > M.M[i][i])
			i = 2;

		static constexpr int32 nxt[3] = { 1, 2, 0 };
		const int32 j = nxt[i];
		const int32 k = nxt[j];
 
		s = M.M[i][i] - M.M[j][j] - M.M[k][k] + T(1.0f);

		T InvS = FMath::InvSqrt(s);

		T qt[4];
		qt[i] = T(0.5f) * (T(1.f) / InvS);

		s = T(0.5f) * InvS;

		qt[3] = (M.M[j][k] - M.M[k][j]) * s;
		qt[j] = (M.M[i][j] + M.M[j][i]) * s;
		qt[k] = (M.M[i][k] + M.M[k][i]) * s;

		this->X = qt[0];
		this->Y = qt[1];
		this->Z = qt[2];
		this->W = qt[3];

		DiagnosticCheckNaN();
	}
}


template<typename T>
inline TQuat<T>::TQuat(const TRotator<T>& R)
{
	*this = R.Quaternion();
	DiagnosticCheckNaN();
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TQuat<T>::operator*(const TVector<T>& V) const
{
	return RotateVector(V);
}

/* TQuat inline functions
 *****************************************************************************/

template<typename T>
UE_FORCEINLINE_HINT TQuat<T>::TQuat(EForceInit ZeroOrNot)
	:	X(0), Y(0), Z(0), W(ZeroOrNot == ForceInitToZero ? 0.0f : 1.0f)
{ }

template<typename T>
inline TQuat<T>::TQuat(T InX, T InY, T InZ, T InW)
	: X(InX)
	, Y(InY)
	, Z(InZ)
	, W(InW)
{
	static_assert(TIsFloatingPoint<T>::Value);
	DiagnosticCheckNaN();
}

template<typename T>
inline TQuat<T>::TQuat(QuatVectorRegister V)
{
	VectorStoreAligned(V, this);
	DiagnosticCheckNaN();
}


template<typename T>
UE_FORCEINLINE_HINT FString TQuat<T>::ToString() const
{
	return FString::Printf(TEXT("X=%.9f Y=%.9f Z=%.9f W=%.9f"), X, Y, Z, W);
}

template<typename T>
inline bool TQuat<T>::InitFromString(const FString& InSourceString)
{
	X = Y = Z = 0.f;
	W = 1.f;

	const bool bSuccess
		=  FParse::Value(*InSourceString, TEXT("X="), X)
		&& FParse::Value(*InSourceString, TEXT("Y="), Y)
		&& FParse::Value(*InSourceString, TEXT("Z="), Z)
		&& FParse::Value(*InSourceString, TEXT("W="), W);
	DiagnosticCheckNaN();
	return bSuccess;
}

template<typename T>
inline TQuat<T>::TQuat(TVector<T> Axis, T AngleRad)
{
	const T half_a = 0.5f * AngleRad;
	T s, c;
	FMath::SinCos(&s, &c, half_a);

	X = s * Axis.X;
	Y = s * Axis.Y;
	Z = s * Axis.Z;
	W = c;

	DiagnosticCheckNaN();
}


template<typename T>
inline TQuat<T> TQuat<T>::operator+(const TQuat<T>& Q) const
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	QuatVectorRegister A = VectorLoadAligned(this);
	QuatVectorRegister B = VectorLoadAligned(&Q);
	return TQuat(VectorAdd(A, B));
#else
	return TQuat(X + Q.X, Y + Q.Y, Z + Q.Z, W + Q.W);
#endif // PLATFORM_ENABLE_VECTORINTRINSICS
}


template<typename T>
inline TQuat<T> TQuat<T>::operator+=(const TQuat<T>& Q)
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	QuatVectorRegister A = VectorLoadAligned(this);
	QuatVectorRegister B = VectorLoadAligned(&Q);
	VectorStoreAligned(VectorAdd(A, B), this);
#else
	this->X += Q.X;
	this->Y += Q.Y;
	this->Z += Q.Z;
	this->W += Q.W;
#endif // PLATFORM_ENABLE_VECTORINTRINSICS

	DiagnosticCheckNaN();

	return *this;
}


template<typename T>
inline TQuat<T> TQuat<T>::operator-(const TQuat<T>& Q) const
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	QuatVectorRegister A = VectorLoadAligned(this);
	QuatVectorRegister B = VectorLoadAligned(&Q);
	return TQuat(VectorSubtract(A, B));
#else
	return TQuat(X - Q.X, Y - Q.Y, Z - Q.Z, W - Q.W);
#endif // PLATFORM_ENABLE_VECTORINTRINSICS
}

template<typename T>
inline TQuat<T> TQuat<T>::operator-() const
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	return TQuat(VectorNegate(VectorLoadAligned(this)));
#else
	return TQuat(-X, -Y, -Z, -W);
#endif
}

template<typename T>
inline bool TQuat<T>::Equals(const TQuat<T>& Q, T Tolerance) const
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	const QuatVectorRegister ToleranceV = VectorLoadFloat1(&Tolerance);
	const QuatVectorRegister A = VectorLoadAligned(this);
	const QuatVectorRegister B = VectorLoadAligned(&Q);

	const QuatVectorRegister RotationSub = VectorAbs(VectorSubtract(A, B));
	const QuatVectorRegister RotationAdd = VectorAbs(VectorAdd(A, B));
	return !VectorAnyGreaterThan(RotationSub, ToleranceV) || !VectorAnyGreaterThan(RotationAdd, ToleranceV);
#else
	return (FMath::Abs(X - Q.X) <= Tolerance && FMath::Abs(Y - Q.Y) <= Tolerance && FMath::Abs(Z - Q.Z) <= Tolerance && FMath::Abs(W - Q.W) <= Tolerance)
		|| (FMath::Abs(X + Q.X) <= Tolerance && FMath::Abs(Y + Q.Y) <= Tolerance && FMath::Abs(Z + Q.Z) <= Tolerance && FMath::Abs(W + Q.W) <= Tolerance);
#endif // PLATFORM_ENABLE_VECTORINTRINSICS
}

template<typename T>
UE_FORCEINLINE_HINT bool TQuat<T>::IsIdentity(T Tolerance) const
{
	return Equals(TQuat<T>::Identity, Tolerance);
}

template<typename T>
inline TQuat<T> TQuat<T>::operator-=(const TQuat<T>& Q)
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	QuatVectorRegister A = VectorLoadAligned(this);
	QuatVectorRegister B = VectorLoadAligned(&Q);
	VectorStoreAligned(VectorSubtract(A, B), this);
#else
	this->X -= Q.X;
	this->Y -= Q.Y;
	this->Z -= Q.Z;
	this->W -= Q.W;
#endif // PLATFORM_ENABLE_VECTORINTRINSICS

	DiagnosticCheckNaN();

	return *this;
}


template<typename T>
inline TQuat<T> TQuat<T>::operator*(const TQuat<T>& Q) const
{
	TQuat<T> Result;
	VectorQuaternionMultiply(&Result, this, &Q);
	
	Result.DiagnosticCheckNaN();
	
	return Result;
}


template<typename T>
inline TQuat<T> TQuat<T>::operator*=(const TQuat<T>& Q)
{
	QuatVectorRegister A = VectorLoadAligned(this);
	QuatVectorRegister B = VectorLoadAligned(&Q);
	QuatVectorRegister Result = VectorQuaternionMultiply2(A, B);
	VectorStoreAligned(Result, this);

	DiagnosticCheckNaN();

	return *this; 
}


// Global operator for (float * Quat)
template<typename T>
UE_FORCEINLINE_HINT TQuat<T> operator*(const float Scale, const TQuat<T>& Q)
{
	return Q.operator*(Scale);
}

// Global operator for (double * Quat)
template<typename T>
UE_FORCEINLINE_HINT TQuat<T> operator*(const double Scale, const TQuat<T>& Q)
{
	return Q.operator*(Scale);
}

template<typename T>
UE_FORCEINLINE_HINT bool TQuat<T>::Identical(const TQuat* Q, const uint32 PortFlags) const
{
	return X == Q->X && Y == Q->Y && Z == Q->Z && W == Q->W;
}

template<typename T>
inline bool TQuat<T>::operator==(const TQuat<T>& Q) const
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	const QuatVectorRegister A = VectorLoadAligned(this);
	const QuatVectorRegister B = VectorLoadAligned(&Q);
	return VectorMaskBits(VectorCompareEQ(A, B)) == 0x0F;
#else
	return X == Q.X && Y == Q.Y && Z == Q.Z && W == Q.W;
#endif // PLATFORM_ENABLE_VECTORINTRINSICS
}


template<typename T>
inline bool TQuat<T>::operator!=(const TQuat<T>& Q) const
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	const QuatVectorRegister A = VectorLoadAligned(this);
	const QuatVectorRegister B = VectorLoadAligned(&Q);
	return VectorMaskBits(VectorCompareNE(A, B)) != 0x00;
#else
	return X != Q.X || Y != Q.Y || Z != Q.Z || W != Q.W;
#endif // PLATFORM_ENABLE_VECTORINTRINSICS
}


template<typename T>
UE_FORCEINLINE_HINT T TQuat<T>::operator|(const TQuat<T>& Q) const
{
	return X * Q.X + Y * Q.Y + Z * Q.Z + W * Q.W;
}


template<typename T>
inline void TQuat<T>::Normalize(T Tolerance)
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	const QuatVectorRegister Vector = VectorLoadAligned(this);

	const QuatVectorRegister SquareSum = VectorDot4(Vector, Vector);
	const QuatVectorRegister NonZeroMask = VectorCompareGE(SquareSum, VectorLoadFloat1(&Tolerance));
	const QuatVectorRegister InvLength = VectorReciprocalSqrtAccurate(SquareSum);
	const QuatVectorRegister NormalizedVector = VectorMultiply(InvLength, Vector);
	QuatVectorRegister Result = VectorSelect(NonZeroMask, NormalizedVector, GlobalVectorConstants::Float0001);

	VectorStoreAligned(Result, this);
#else
	const T SquareSum = X * X + Y * Y + Z * Z + W * W;

	if (SquareSum >= Tolerance)
	{
		const T Scale = FMath::InvSqrt(SquareSum);

		X *= Scale; 
		Y *= Scale; 
		Z *= Scale;
		W *= Scale;
	}
	else
	{
		*this = TQuat<T>::Identity;
	}
#endif // PLATFORM_ENABLE_VECTORINTRINSICS
}


template<typename T>
inline TQuat<T> TQuat<T>::GetNormalized(T Tolerance) const
{
	TQuat<T> Result(*this);
	Result.Normalize(Tolerance);
	return Result;
}


template<typename T>
inline bool TQuat<T>::IsNormalized() const
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	QuatVectorRegister A = VectorLoadAligned(this);
	QuatVectorRegister TestValue = VectorAbs(VectorSubtract(VectorOne(), VectorDot4(A, A)));
	return !VectorAnyGreaterThan(TestValue, GlobalVectorConstants::ThreshQuatNormalized);
#else
	return (FMath::Abs(1.f - SizeSquared()) < UE_THRESH_QUAT_NORMALIZED);
#endif // PLATFORM_ENABLE_VECTORINTRINSICS
}


template<typename T>
UE_FORCEINLINE_HINT T TQuat<T>::Size() const
{
	return FMath::Sqrt(X * X + Y * Y + Z * Z + W * W);
}

template<typename T>
UE_FORCEINLINE_HINT T TQuat<T>::SizeSquared() const
{
	return (X * X + Y * Y + Z * Z + W * W);
}

template<typename T>
UE_FORCEINLINE_HINT T TQuat<T>::GetAngle() const
{
	return T(2.0) * FMath::Acos(W);
}


template<typename T>
inline void TQuat<T>::ToAxisAndAngle(TVector<T>& Axis, float& Angle) const
{
	Angle = (float)GetAngle();
	Axis = GetRotationAxis();
}

template<typename T>
inline void TQuat<T>::ToAxisAndAngle(TVector<T>& Axis, double& Angle) const
{
	Angle = (double)GetAngle();
	Axis = GetRotationAxis();
}

template<typename T>
inline TVector<T> TQuat<T>::ToRotationVector() const
{
	checkSlow(IsNormalized());
	TQuat<T> RotQ = Log();
	return TVector<T>(RotQ.X * 2.0f, RotQ.Y * 2.0f, RotQ.Z * 2.0f);
}

template<typename T>
inline TQuat<T> TQuat<T>::MakeFromRotationVector(const TVector<T>& RotationVector)
{
	TQuat<T> RotQ(RotationVector.X * 0.5f, RotationVector.Y * 0.5f, RotationVector.Z * 0.5f, 0.0f);
	return RotQ.Exp();
}

template<typename T>
inline TVector<T> TQuat<T>::GetRotationAxis() const
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	TVector<T> V;
	QuatVectorRegister A = VectorLoadAligned(this);
	QuatVectorRegister R = VectorNormalizeSafe(VectorSet_W0(A), GlobalVectorConstants::Float1000);
	VectorStoreFloat3(R, &V);
	return V;
#else
	const T SquareSum = X * X + Y * Y + Z * Z;
	if (SquareSum < UE_SMALL_NUMBER)
	{
		return TVector<T>::XAxisVector;
	}
	const T Scale = FMath::InvSqrt(SquareSum);
	return TVector<T>(X * Scale, Y * Scale, Z * Scale);
#endif // PLATFORM_ENABLE_VECTORINTRINSICS
}

template<typename T>
T TQuat<T>::AngularDistance(const TQuat<T>& Q) const
{
	T InnerProd = X*Q.X + Y*Q.Y + Z*Q.Z + W*Q.W;
	return FMath::Acos((2 * InnerProd * InnerProd) - 1.f);
}


template<typename T>
inline TVector<T> TQuat<T>::RotateVector(TVector<T> V) const
{	
	// http://people.csail.mit.edu/bkph/articles/Quaternions.pdf
	// V' = V + 2w(Q x V) + (2Q x (Q x V))
	// refactor:
	// V' = V + w(2(Q x V)) + (Q x (2(Q x V)))
	// T = 2(Q x V);
	// V' = V + w*(T) + (Q x T)

	const TVector<T> Q(X, Y, Z);
	const TVector<T> TT = 2.f * TVector<T>::CrossProduct(Q, V);
	const TVector<T> Result = V + (W * TT) + TVector<T>::CrossProduct(Q, TT);
	return Result;
}

template<typename T>
inline TVector<T> TQuat<T>::UnrotateVector(TVector<T> V) const
{	
	const TVector<T> Q(-X, -Y, -Z); // Inverse
	const TVector<T> TT = 2.f * TVector<T>::CrossProduct(Q, V);
	const TVector<T> Result = V + (W * TT) + TVector<T>::CrossProduct(Q, TT);
	return Result;
}


template<typename T>
inline TQuat<T> TQuat<T>::Inverse() const
{
	checkSlow(IsNormalized());

#if PLATFORM_ENABLE_VECTORINTRINSICS
	return TQuat(VectorQuaternionInverse(VectorLoadAligned(this)));
#else
	return TQuat(-X, -Y, -Z, W);
#endif // PLATFORM_ENABLE_VECTORINTRINSICS
}

template<typename T>
inline void TQuat<T>::EnforceShortestArcWith(const TQuat<T>& OtherQuat)
{
	const T DotResult = (OtherQuat | *this);
	const T Bias = FMath::FloatSelect(DotResult, T(1.0f), T(-1.0f));

	X *= Bias;
	Y *= Bias;
	Z *= Bias;
	W *= Bias;
}

template<typename T>
inline TQuat<T> TQuat<T>::GetShortestArcWith(const TQuat<T>& OtherQuat) const
{
	TQuat<T> Result(*this);
	Result.EnforceShortestArcWith(OtherQuat);
	return Result;
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TQuat<T>::GetAxisX() const
{
	return RotateVector(TVector<T>(1.f, 0.f, 0.f));
}


template<typename T>
UE_FORCEINLINE_HINT TVector<T> TQuat<T>::GetAxisY() const
{
	return RotateVector(TVector<T>(0.f, 1.f, 0.f));
}


template<typename T>
UE_FORCEINLINE_HINT TVector<T> TQuat<T>::GetAxisZ() const
{
	return RotateVector(TVector<T>(0.f, 0.f, 1.f));
}


template<typename T>
UE_FORCEINLINE_HINT TVector<T> TQuat<T>::GetForwardVector() const
{
	return GetAxisX();
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TQuat<T>::GetRightVector() const
{
	return GetAxisY();
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TQuat<T>::GetUpVector() const
{
	return GetAxisZ();
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TQuat<T>::Vector() const
{
	return GetAxisX();
}

template<typename T>
inline TMatrix<T> TQuat<T>::ToMatrix() const
{
	TMatrix<T> R;
	ToMatrix(R);
	return R;
}

template<typename T>
inline T TQuat<T>::Error(const TQuat<T>& Q1, const TQuat<T>& Q2)
{
	const T cosom = FMath::Abs(Q1.X * Q2.X + Q1.Y * Q2.Y + Q1.Z * Q2.Z + Q1.W * Q2.W);
	return (FMath::Abs(cosom) < 0.9999999f) ? FMath::Acos(cosom)*(1.f / UE_PI) : 0.0f;
}


template<typename T>
inline T TQuat<T>::ErrorAutoNormalize(const TQuat<T>& A, const TQuat<T>& B)
{
	TQuat<T> Q1 = A;
	Q1.Normalize();

	TQuat<T> Q2 = B;
	Q2.Normalize();

	return TQuat<T>::Error(Q1, Q2);
}

/**
 * Fast Linear Quaternion Interpolation.
 * Result is NOT normalized.
 */
template<typename T>
inline TQuat<T> TQuat<T>::FastLerp(const TQuat<T>& A, const TQuat<T>& B, const T Alpha)
{
	// To ensure the 'shortest route', we make sure the dot product between the both rotations is positive.
	const T DotResult = (A | B);
	const T Bias = FMath::FloatSelect(DotResult, T(1.0f), T(-1.0f));
	return (B * Alpha) + (A * (Bias * (1.f - Alpha)));
}


template<typename T>
inline TQuat<T> TQuat<T>::FastBilerp(const TQuat<T>& P00, const TQuat<T>& P10, const TQuat<T>& P01, const TQuat<T>& P11, T FracX, T FracY)
{
	return TQuat<T>::FastLerp(
		TQuat<T>::FastLerp(P00,P10,FracX),
		TQuat<T>::FastLerp(P01,P11,FracX),
		FracY
	);
}


template<typename T>
inline bool TQuat<T>::ContainsNaN() const
{
	return (!FMath::IsFinite(X) ||
			!FMath::IsFinite(Y) ||
			!FMath::IsFinite(Z) ||
			!FMath::IsFinite(W)
	);
}
	
/**
 * Creates a hash value from an FQuat.
 *
 * @param Quat the quat to create a hash value for
 * @return The hash value from the components
 */
template<typename T>
inline uint32 GetTypeHash(const TQuat<T>& Quat)
{
	// Note: this assumes there's no padding in Quat that could contain uncompared data.
	static_assert(sizeof(TQuat<T>) == sizeof(T[4]), "Unexpected padding in TQuat");
	return FCrc::MemCrc_DEPRECATED(&Quat, sizeof(Quat));
}

template<typename T>
[[nodiscard]] TTuple<T, T, T> TQuat<T>::ToLUFEuler() const
{
	auto ThreeAxisRot = [](T R11, T R12, T R21, T R31, T R32, T Euler[])
	{
		Euler[0] = FMath::Atan2(R31, R32);
		Euler[1] = FMath::Asin(R21);
		Euler[2] = FMath::Atan2(R11, R12);
	};
	
	T Euler[3];
	ThreeAxisRot(2*(Y*Z + W*X),
				 W*W - X*X + Y*Y - Z*Z,
				 -2*(X*Y - W*Z),
				 2*(X*Z + W*Y),
				 W*W + X*X - Y*Y - Z*Z,
				 Euler);
	// Going from left-handed to right-handed
	return {Euler[0], -Euler[1], -Euler[2]};
}

template<typename T>
[[nodiscard]] TQuat<T> TQuat<T>::MakeFromLUFEuler(const TTuple<T, T, T>& InLUFEuler)
{
	const T LeftAxisAngle = InLUFEuler.template Get<0>();
	const T UpAxisAngle = InLUFEuler.template Get<1>();
	const T ForwardAxisAngle = InLUFEuler.template Get<2>();
		
	// Going from left-handed to right-handed
	TQuat LeftRotation(TVector<T>::LeftVector, -LeftAxisAngle);
	TQuat UpRotation(TVector<T>::UpVector, -UpAxisAngle);
	TQuat ForwardRotation(TVector<T>::ForwardVector, -ForwardAxisAngle);
	return ForwardRotation * UpRotation * LeftRotation;
}

} // namespace UE::Math
} // namespace UE

UE_DECLARE_LWC_TYPE(Quat, 4);

template<> struct TIsPODType<FQuat4f> { enum { Value = true }; };
template<> struct TIsPODType<FQuat4d> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FQuat4f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FQuat4d> { enum { Value = true }; };
template<> struct TCanBulkSerialize<FQuat4f> { enum { Value = true }; };
template<> struct TCanBulkSerialize<FQuat4d> { enum { Value = true }; };
DECLARE_INTRINSIC_TYPE_LAYOUT(FQuat4f);
DECLARE_INTRINSIC_TYPE_LAYOUT(FQuat4d);

// Forward declare all explicit specializations (in UnrealMath.cpp)
template<> CORE_API FRotator3f FQuat4f::Rotator() const;
template<> CORE_API FRotator3d FQuat4d::Rotator() const;


/* FMath inline functions
 *****************************************************************************/
 
 // TCustomLerp for FMath::Lerp()
template<typename T>
struct TCustomLerp< UE::Math::TQuat<T> >
{
	constexpr static bool Value = true;
	using QuatType = UE::Math::TQuat<T>;

	template<class U>
	static UE_FORCEINLINE_HINT QuatType Lerp(const QuatType& A, const QuatType& B, const U& Alpha)
	{
		return QuatType::Slerp(A, B, (T)Alpha);
	}

	template<class U>
	static inline QuatType BiLerp(const QuatType& P00, const QuatType& P10, const QuatType& P01, const QuatType& P11, const U& FracX, const U& FracY)
	{
		QuatType Result;

		Result = Lerp(
			QuatType::Slerp_NotNormalized(P00, P10, (T)FracX),
			QuatType::Slerp_NotNormalized(P01, P11, (T)FracX),
			(T)FracY
		);

		return Result;
	}

	template<class U>
	static UE_FORCEINLINE_HINT QuatType CubicInterp(const QuatType& P0, const QuatType& T0, const QuatType& P1, const QuatType& T1, const U& A)
	{
		return QuatType::Squad(P0, T0, P1, T1, (T)A);
	}

};
