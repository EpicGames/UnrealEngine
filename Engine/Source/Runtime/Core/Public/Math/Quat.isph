// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __QUAT_ISPH__
#define __QUAT_ISPH__

#include "Math/Vector.isph"
#include "Math/Matrix.isph"

inline uniform FVector4 MatrixToQuat(const uniform FMatrix &M)
{
	// If Matrix is NULL, return Identity quaternion. If any of them is 0, you won't be able to construct rotation
	// if you have two plane at least, we can reconstruct the frame using cross product, but that's a bit expensive op to do here
	// for now, if you convert to matrix from 0 scale and convert back, you'll lose rotation. Don't do that.
	if (IsAnyMatrixScaledAxesNearlyZero(M, KINDA_SMALL_NUMBER))
	{
		return Vector0001;
	}

	//const MeReal *const t = (MeReal *) tm;
	uniform FReal	s;

	// Check diagonal (trace)
	const uniform FReal tr = M.M[0] + M.M[5] + M.M[10];

	if (tr > ZERO)
	{
		uniform FVector4 qt;
		const uniform FReal InvS = ONE / sqrt(tr + 1);
		s = ONE_HALF * InvS;

		qt.V[0] = (M.M[6] - M.M[9]) * s;
		qt.V[1] = (M.M[8] - M.M[2]) * s;
		qt.V[2] = (M.M[1] - M.M[4]) * s;
		qt.V[3] = ONE_HALF * (ONE / InvS);

		return qt;
	}
	else
	{
		// diagonal is negative
		uniform int i = 0;

		if (M.M[5] > M.M[0])
		{
			i = 1;
		}

		if (M.M[10] > M.M[4*i+i])
		{
			i = 2;
		}

		static const uniform int nxt[3] = { 1, 2, 0 };
		const uniform int j = nxt[i];
		const uniform int k = nxt[j];

		s = M.M[4*i+i] - M.M[4*j+j] - M.M[4*k+k] + ONE;

		const uniform FReal InvS = ONE / sqrt(s);
		s = ONE_HALF * InvS;

		uniform FVector4 qt;

		qt.V[i] = ONE_HALF * (ONE / InvS);
		qt.V[j] = (M.M[4*i+j] + M.M[4*j+i]) * s;
		qt.V[k] = (M.M[4*i+k] + M.M[4*k+i]) * s;
		qt.V[3] = (M.M[4*j+k] - M.M[4*k+j]) * s;

		return qt;
	}
}

/**
* Multiplies two quaternions; the order matters.
*
* Order matters when composing quaternions: C = VectorQuaternionMultiply2(A, B) will yield a quaternion C = A * B
* that logically first applies B then A to any subsequent transformation (right first, then left).
*
* @param Quat1	Pointer to the first quaternion
* @param Quat2	Pointer to the second quaternion
* @return Quat1 * Quat2
*/
template <typename T>
inline T VectorQuaternionMultiply2(const T& Quat1, const T& Quat2)
{
	T Result = VectorReplicate(Quat1, 3) * Quat2;
	Result = VectorMultiplyAdd((VectorReplicate(Quat1, 0) * VectorSwizzle(Quat2, 3,2,1,0)), QMULTI_SIGN_MASK0, Result);
	Result = VectorMultiplyAdd((VectorReplicate(Quat1, 1) * VectorSwizzle(Quat2, 2,3,0,1)), QMULTI_SIGN_MASK1, Result);
	Result = VectorMultiplyAdd((VectorReplicate(Quat1, 2) * VectorSwizzle(Quat2, 1,0,3,2)), QMULTI_SIGN_MASK2, Result);

	return Result;
}

// Normalize quaternion ( result = (Q.Q >= 1e-8) ? (Q / |Q|) : (0,0,0,1) )
inline FVector4d VectorNormalizeQuaternion(const FVector4d& UnnormalizedQuat)
{
	return VectorNormalizeSafe(UnnormalizedQuat, Double0001);
}

inline FVector4f VectorNormalizeQuaternion(const FVector4f& UnnormalizedQuat)
{
	return VectorNormalizeSafe(UnnormalizedQuat, Float0001);
}

inline uniform FVector4 VectorNormalizeQuaternion(const uniform FVector4& UnnormalizedQuat)
{
	return VectorNormalizeSafe2(UnnormalizedQuat, Vector0001);
}

inline FVector4 QuatInverse(const FVector4 &Quat)
{
	return Quat * QINV_SIGN_MASK;
}

inline uniform FVector4d QuatInverse(const uniform FVector4d &Quat)
{
	return Quat * DOUBLE_QINV_SIGN_MASK;
}

inline uniform FVector4f QuatInverse(const uniform FVector4f &Quat)
{
	return Quat * FLOAT_QINV_SIGN_MASK;
}

template <typename T, typename F>
inline T QuatFastLerp(const T& A, const T& B, const F Alpha)
{
	// To ensure the 'shortest route', we make sure the dot product between the both rotations is positive.
	const F DotResult = VectorDot(A, B);
	const F Bias = select(DotResult >= 0, 1, -1);
	return (B * Alpha) + (A * (Bias * (1 - Alpha)));
}

// A and B are quaternions.  The result is A + (|A.B| >= 0 ? 1 : -1) * B
inline uniform FVector4 VectorAccumulateQuaternionShortestPath(const uniform FVector4& A, const uniform FVector4& B)
{
	// Blend rotation
	//     To ensure the 'shortest route', we make sure the dot product between the both rotations is positive.
	//     const float Bias = (|A.B| >= 0 ? 1 : -1)
	//     return A + B * Bias;
	const uniform FVector4 RotationDot = VectorDot4(A, B);
	const uniform FVector4 QuatRotationDirMask = VectorCompareGE(RotationDot, VectorZero);
	const uniform FVector4 NegativeB = VectorZero - B;
	const uniform FVector4 BiasTimesB = VectorSelect(QuatRotationDirMask, B, NegativeB);
	return A + BiasTimesB;
}

/**
 * Rotate a vector using a unit Quaternion.
 *
 * @param Quat Unit Quaternion to use for rotation.
 * @param VectorW0 Vector to rotate. W component must be zero.
 * @return Vector after rotation by Quat.
 */
template<typename T, typename V>
inline V VectorQuaternionRotateVector(const T& Quat, const V& VectorW0)
{
	// Q * V * Q.Inverse
	//const VectorRegister InverseRotation = VectorQuaternionInverse(Quat);
	//const VectorRegister Temp = VectorQuaternionMultiply2(Quat, VectorW0);
	//const VectorRegister Rotated = VectorQuaternionMultiply2(Temp, InverseRotation);

	// Equivalence of above can be shown to be:
	// http://people.csail.mit.edu/bkph/articles/Quaternions.pdf
	// V' = V + 2w(Q x V) + (2Q x (Q x V))
	// refactor:
	// V' = V + w(2(Q x V)) + (Q x (2(Q x V)))
	// T = 2(Q x V);
	// V' = V + w*(T) + (Q x T)

	const V QW = VectorReplicate(Quat, 3);
	V Q = VectorCross(Quat, VectorW0);
	Q = VectorAdd(Q, Q);
	const V VTemp0 = VectorMultiplyAdd(QW, Q, VectorW0);
	const V VTemp1 = VectorCross(Quat, Q);
	const V Rotated = VectorAdd(VTemp0, VTemp1);
	return Rotated;
}

template<>
inline uniform FVector VectorQuaternionRotateVector<uniform FVector4, uniform FVector>(const uniform FVector4& Quat, const uniform FVector& V)
{
	// http://people.csail.mit.edu/bkph/articles/Quaternions.pdf
	// V' = V + 2w(Q x V) + (2Q x (Q x V))
	// refactor:
	// V' = V + w(2(Q x V)) + (Q x (2(Q x V)))
	// T = 2(Q x V);
	// V' = V + w*(T) + (Q x T)

	const uniform FVector Q = SetVector(Quat.V[0], Quat.V[1], Quat.V[2]);
	const uniform FVector T = VectorCross(Q, V) * 2;
	const uniform FVector Result = V + (T * Quat.V[3]) + VectorCross(Q, T);
	return Result;
}

template<>
inline FVector3d VectorQuaternionRotateVector<FVector4d, FVector3d>(const FVector4d& Quat, const FVector3d& V)
{
	// http://people.csail.mit.edu/bkph/articles/Quaternions.pdf
	// V' = V + 2w(Q x V) + (2Q x (Q x V))
	// refactor:
	// V' = V + w(2(Q x V)) + (Q x (2(Q x V)))
	// T = 2(Q x V);
	// V' = V + w*(T) + (Q x T)

	const FVector3d Q = SetVector(Quat.V[0], Quat.V[1], Quat.V[2]);
	const FVector3d T = VectorCross(Q, V) * 2;
	const FVector3d Result = V + (T * Quat.V[3]) + VectorCross(Q, T);
	return Result;
}

template<>
inline FVector3f VectorQuaternionRotateVector<FVector4f, FVector3f>(const FVector4f& Quat, const FVector3f& V)
{
	// http://people.csail.mit.edu/bkph/articles/Quaternions.pdf
	// V' = V + 2w(Q x V) + (2Q x (Q x V))
	// refactor:
	// V' = V + w(2(Q x V)) + (Q x (2(Q x V)))
	// T = 2(Q x V);
	// V' = V + w*(T) + (Q x T)

	const FVector3f Q = SetVector(Quat.V[0], Quat.V[1], Quat.V[2]);
	const FVector3f T = VectorCross(Q, V) * 2;
	const FVector3f Result = V + (T * Quat.V[3]) + VectorCross(Q, T);
	return Result;
}

template<>
inline FVector3d VectorQuaternionRotateVector<uniform FVector4d, FVector3d>(const uniform FVector4d& Quat, const FVector3d& V)
{
	// http://people.csail.mit.edu/bkph/articles/Quaternions.pdf
	// V' = V + 2w(Q x V) + (2Q x (Q x V))
	// refactor:
	// V' = V + w(2(Q x V)) + (Q x (2(Q x V)))
	// T = 2(Q x V);
	// V' = V + w*(T) + (Q x T)

	const uniform FVector3d Q = SetVector(Quat.V[0], Quat.V[1], Quat.V[2]);
	const FVector3d T = VectorCross(Q, V) * 2;
	const FVector3d Result = V + (T * Quat.V[3]) + VectorCross(Q, T);
	return Result;
}

template<>
inline FVector3f VectorQuaternionRotateVector<uniform FVector4f, FVector3f>(const uniform FVector4f& Quat, const FVector3f& V)
{
	// http://people.csail.mit.edu/bkph/articles/Quaternions.pdf
	// V' = V + 2w(Q x V) + (2Q x (Q x V))
	// refactor:
	// V' = V + w(2(Q x V)) + (Q x (2(Q x V)))
	// T = 2(Q x V);
	// V' = V + w*(T) + (Q x T)

	const uniform FVector3f Q = SetVector(Quat.V[0], Quat.V[1], Quat.V[2]);
	const FVector3f T = VectorCross(Q, V) * 2;
	const FVector3f Result = V + (T * Quat.V[3]) + VectorCross(Q, T);
	return Result;
}

template<>
inline uniform FVector8 VectorQuaternionRotateVector<uniform FVector8, uniform FVector8>(const uniform FVector8& Quat, const uniform FVector8& VectorW0)
{
	const uniform FVector8 QW = VectorReplicate(Quat, 3);
	uniform FVector8 T = VectorCross(Quat, VectorW0);
	T = VectorAdd(T, T);
	const uniform FVector8 VTemp0 = VectorMultiplyAdd(QW, T, VectorW0);
	const uniform FVector8 VTemp1 = VectorCross(Quat, T);
	const uniform FVector8 Rotated = VectorAdd(VTemp0, VTemp1);
	return Rotated;
}

inline uniform FVector4 VectorLerpQuat(const uniform FVector4& A, const uniform FVector4& B, const uniform FReal Alpha)
{
	// Blend rotation
	//     To ensure the 'shortest route', we make sure the dot product between the both rotations is positive.
	//     const float Bias = (|A.B| >= 0 ? 1 : -1)
	//     Rotation = (B * Alpha) + (A * (Bias * (1.f - Alpha)));
	const uniform FReal OneMinusAlpha = ONE - Alpha;

	const uniform FVector4 RotationDot = VectorDot4(A, B);
	const uniform FVector4 QuatRotationDirMask = VectorCompareGE(RotationDot, VectorZero);
	const uniform FVector4 NegativeA = VectorSubtract(VectorZero, A);
	const uniform FVector4 BiasTimesA = VectorSelect(QuatRotationDirMask, A, NegativeA);
	const uniform FVector4 BTimesWeight = VectorMultiply(B, Alpha);
	const uniform FVector4 UnnormalizedResult = VectorMultiplyAdd(BiasTimesA, OneMinusAlpha, BTimesWeight);

	return UnnormalizedResult;
}

inline uniform FVector4 VectorQuaternionEnforceShortestArcWith(const uniform FVector4& A, const uniform FVector4& B)
{
	const uniform FReal DotResult = VectorDot(B, A);
	const uniform FReal Bias = select(DotResult >= ZERO, ONE, -ONE);

	return A * Bias;
}

#endif
