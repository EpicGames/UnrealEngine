// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Math/Vector.h"
#include "Math/Matrix.h"
#include "Math/Quat.h"

namespace UE {
namespace Math {

/** Rotation and translation matrix using quaternion rotation */
template <typename T>
struct TQuatRotationTranslationMatrix
	: public TMatrix<T>
{
public:
	using TMatrix<T>::M;

	/** Constructor
	*
	* @param Q rotation
	* @param Origin translation to apply
	*/
	TQuatRotationTranslationMatrix(const TQuat<T>& Q, const TVector<T>& Origin);

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TQuatRotationTranslationMatrix(const TQuatRotationTranslationMatrix<FArg>& From)
		: TMatrix<T>(From)
	{
	}

	/** Matrix factory. Return an FMatrix so we don't have type conversion issues in expressions. */
	static TMatrix<T> Make(const TQuat<T>& Q, const TVector<T>& Origin)
	{
		return TQuatRotationTranslationMatrix<T>(Q, Origin);
	}
};


/** Rotation matrix using quaternion rotation */
template<typename T>
struct TQuatRotationMatrix
	: public TQuatRotationTranslationMatrix<T>
{
public:
	using TQuatRotationTranslationMatrix<T>::M;

	/** Constructor
	*
	* @param Q rotation
	*/
	TQuatRotationMatrix(const TQuat<T>& Q)
		: TQuatRotationTranslationMatrix<T>(Q, TVector<T>::ZeroVector)
	{
	}

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TQuatRotationMatrix(const TQuatRotationMatrix<FArg>& From)
		: TQuatRotationTranslationMatrix<T>(From)
	{
	}
	
	/** Matrix factory. Return an FMatrix so we don't have type conversion issues in expressions. */
	static TMatrix<T> Make(const TQuat<T>& Q)
	{
		return TQuatRotationMatrix<T>(Q);
	}
};


template<typename T>
inline TQuatRotationTranslationMatrix<T>::TQuatRotationTranslationMatrix(const TQuat<T>& Q, const TVector<T>& Origin)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_EDITORONLY_DATA
	// Make sure Quaternion is normalized
	check( Q.IsNormalized() );
#endif
	const T x2 = Q.X + Q.X;  const T y2 = Q.Y + Q.Y;  const T z2 = Q.Z + Q.Z;
	const T xx = Q.X * x2;   const T xy = Q.X * y2;   const T xz = Q.X * z2;
	const T yy = Q.Y * y2;   const T yz = Q.Y * z2;   const T zz = Q.Z * z2;
	const T wx = Q.W * x2;   const T wy = Q.W * y2;   const T wz = Q.W * z2;

	M[0][0] = 1.0f - (yy + zz);	M[1][0] = xy - wz;				M[2][0] = xz + wy;			M[3][0] = Origin.X;
	M[0][1] = xy + wz;			M[1][1] = 1.0f - (xx + zz);		M[2][1] = yz - wx;			M[3][1] = Origin.Y;
	M[0][2] = xz - wy;			M[1][2] = yz + wx;				M[2][2] = 1.0f - (xx + yy);	M[3][2] = Origin.Z;
	M[0][3] = 0.0f;				M[1][3] = 0.0f;					M[2][3] = 0.0f;				M[3][3] = 1.0f;
}

} // namespace Math
} // namespace UE

UE_DECLARE_LWC_TYPE(QuatRotationTranslationMatrix, 44);
UE_DECLARE_LWC_TYPE(QuatRotationMatrix, 44);

template<> struct TIsUECoreVariant<FQuatRotationTranslationMatrix44f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FQuatRotationTranslationMatrix44d> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FQuatRotationMatrix44f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FQuatRotationMatrix44d> { enum { Value = true }; };