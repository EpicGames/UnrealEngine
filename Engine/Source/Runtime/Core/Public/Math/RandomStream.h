// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Box.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "Math/Matrix.h"
#include "Math/RotationMatrix.h"
#include "Math/Transform.h"
#include "HAL/PlatformTime.h"

/**
 * Implements a thread-safe SRand based RNG.
 *
 * Very bad quality in the lower bits. Don't use the modulus (%) operator.
 */
struct FRandomStream
{
	friend struct Z_Construct_UScriptStruct_FRandomStream_Statics;
	
public:

	/**
	 * Default constructor.
	 *
	 * The seed should be set prior to use.
	 */
	FRandomStream()
		: InitialSeed(0)
		, Seed(0)
	{ }

	/**
	 * Creates and initializes a new random stream from the specified seed value.
	 *
	 * @param InSeed The seed value.
	 */
	FRandomStream( int32 InSeed )
	{ 
		Initialize(InSeed);
	}

	/**
	 * Creates and initializes a new random stream from the specified name.
	 *
	 * @note If NAME_None is provided, the stream will be seeded using the current time.
	 * @param InName The name value from which the stream will be initialized.
	 */
	FRandomStream( FName InName )
	{
		Initialize(InName);
	}

public:

	/**
	 * Initializes this random stream with the specified seed value.
	 *
	 * @param InSeed The seed value.
	 */
	void Initialize( int32 InSeed )
	{
		InitialSeed = InSeed;
		Seed = uint32(InSeed);
	}

	/**
	 * Initializes this random stream using the specified name.
	 *
	 * @note If NAME_None is provided, the stream will be seeded using the current time.
	 * @param InName The name value from which the stream will be initialized.
	 */
	void Initialize( FName InName )
	{
		if (InName != NAME_None)
		{
			InitialSeed = GetTypeHash(InName.ToString());
		}
		else
		{
			InitialSeed = FPlatformTime::Cycles();
		}

		Seed = uint32(InitialSeed);
	}

	/**
	 * Resets this random stream to the initial seed value.
	 */
	void Reset() const
	{
		Seed = uint32(InitialSeed);
	}

	int32 GetInitialSeed() const
	{
		return InitialSeed;
	}

	/**
	 * Generates a new random seed.
	 */
	void GenerateNewSeed()
	{
		Initialize(FMath::Rand());
	}

	/**
	 * Returns a random float number in the range [0, 1).
	 *
	 * @return Random number.
	 */
	float GetFraction() const
	{
		MutateSeed();

		float Result;

		*(uint32*)&Result = 0x3F800000U | (Seed >> 9);

		return Result - 1.0f; 
	}

	/**
	 * Returns a random number between 0 and MAXUINT.
	 *
	 * @return Random number.
	 */
	uint32 GetUnsignedInt() const
	{
		MutateSeed();

		return Seed;
	}

	/**
	 * Returns a random vector of unit size.
	 *
	 * @return Random unit vector.
	 */
	FVector GetUnitVector() const
	{
		FVector Result;
		FVector::FReal L;

		do
		{
			// Check random vectors in the unit sphere so result is statistically uniform.
			Result.X = GetFraction() * 2.f - 1.f;
			Result.Y = GetFraction() * 2.f - 1.f;
			Result.Z = GetFraction() * 2.f - 1.f;
			L = Result.SizeSquared();
		}
		while(L > 1.f || L < UE_KINDA_SMALL_NUMBER);

		return Result.GetUnsafeNormal();
	}

	/**
	 * Gets the current seed.
	 *
	 * @return Current seed.
	 */
	int32 GetCurrentSeed() const
	{
		return int32(Seed);
	}

	/**
	 * Mirrors the random number API in FMath
	 *
	 * @return Random number.
	 */
	UE_FORCEINLINE_HINT float FRand() const
	{
		return GetFraction();
	}

	/**
	 * Helper function for rand implementations.
	 *
	 * @return A random number in [0..A)
	 */
	UE_FORCEINLINE_HINT int32 RandHelper( int32 A ) const
	{
		// GetFraction guarantees a result in the [0,1) range.
		return ((A > 0) ? FMath::TruncToInt(GetFraction() * float(A)) : 0);
	}

	/**
	 * Helper function for rand implementations.
	 *
	 * @return A random number in [0..A)
	 */
	UE_FORCEINLINE_HINT int64 RandHelper64(int64 A) const
	{
		// Note that on some platforms RAND_MAX is a large number so we cannot do ((rand()/(RAND_MAX+1)) * A)
		// or else we may include the upper bound results, which should be excluded.
		return A > 0 ? FMath::Min<int64>(FMath::TruncToInt(GetFraction() * float(A)), A - 1) : 0;
	}

	/**
	 * Helper function for rand implementations.
	 *
	 * @return A random number >= Min and <= Max
	 */
	inline int32 RandRange( int32 Min, int32 Max ) const
	{
		const int32 Range = (Max - Min) + 1;

		return Min + RandHelper(Range);
	}

	/**
	 * Helper function for rand implementations.
	 *
	 * @return A random number >= Min and <= Max
	 */
	UE_FORCEINLINE_HINT FVector::FReal FRandRange( FVector::FReal InMin, FVector::FReal InMax ) const
	{
		return InMin + (InMax - InMin) * FRand();
	}

	/**
	 * Helper function for rand implementations.
	 *
	 * @return A random boolean value.
	 */
	UE_FORCEINLINE_HINT bool RandBool() const
	{
		return RandHelper(2) == 1;
	}

	/**
	 * Returns a random vector of unit size.
	 *
	 * @return Random unit vector.
	 */
	UE_FORCEINLINE_HINT FVector VRand() const
	{
		return GetUnitVector();
	}

	inline FVector RandPointInBox(const FBox& Box) const
	{
		return FVector(	FRandRange(Box.Min.X, Box.Max.X),
						FRandRange(Box.Min.Y, Box.Max.Y),
						FRandRange(Box.Min.Z, Box.Max.Z) );
	}

	/**
	 * Returns a random unit vector, uniformly distributed, within the specified cone.
	 *
	 * @param Dir The center direction of the cone
	 * @param ConeHalfAngleRad Half-angle of cone, in radians.
	 * @return Normalized vector within the specified cone.
	 */
	inline FVector VRandCone( FVector const& Dir, float ConeHalfAngleRad ) const
	{
		if (ConeHalfAngleRad > 0.f)
		{
			float const RandU = FRand();
			float const RandV = FRand();

			// Get spherical coords that have an even distribution over the unit sphere
			// Method described at http://mathworld.wolfram.com/SpherePointPicking.html	
			float Theta = 2.f * UE_PI * RandU;
			float Phi = FMath::Acos((2.f * RandV) - 1.f);

			// restrict phi to [0, ConeHalfAngleRad]
			// this gives an even distribution of points on the surface of the cone
			// centered at the origin, pointing upward (z), with the desired angle
			Phi = FMath::Fmod(Phi, ConeHalfAngleRad);

			// get axes we need to rotate around
			FMatrix const DirMat = FRotationMatrix(Dir.Rotation());
			// note the axis translation, since we want the variation to be around X
			FVector const DirZ = DirMat.GetUnitAxis( EAxis::X );		
			FVector const DirY = DirMat.GetUnitAxis( EAxis::Y );

			FVector Result = Dir.RotateAngleAxis(Phi * 180.f / UE_PI, DirY);
			Result = Result.RotateAngleAxis(Theta * 180.f / UE_PI, DirZ);

			// ensure it's a unit vector (might not have been passed in that way)
			Result = Result.GetSafeNormal();
		
			return Result;
		}
		else
		{
			return Dir.GetSafeNormal();
		}
	}

	/**
	 * Returns a random unit vector, uniformly distributed, within the specified cone.
	 *
	 * @param Dir The center direction of the cone
	 * @param HorizontalConeHalfAngleRad Horizontal half-angle of cone, in radians.
	 * @param VerticalConeHalfAngleRad Vertical half-angle of cone, in radians.
	 * @return Normalized vector within the specified cone.
	 */
	inline FVector VRandCone( FVector const& Dir, float HorizontalConeHalfAngleRad, float VerticalConeHalfAngleRad ) const
	{
		if ( (VerticalConeHalfAngleRad > 0.f) && (HorizontalConeHalfAngleRad > 0.f) )
		{
			float const RandU = FRand();
			float const RandV = FRand();

			// Get spherical coords that have an even distribution over the unit sphere
			// Method described at http://mathworld.wolfram.com/SpherePointPicking.html	
			float Theta = 2.f * UE_PI * RandU;
			float Phi = FMath::Acos((2.f * RandV) - 1.f);

			// restrict phi to [0, ConeHalfAngleRad]
			// where ConeHalfAngleRad is now a function of Theta
			// (specifically, radius of an ellipse as a function of angle)
			// function is ellipse function (x/a)^2 + (y/b)^2 = 1, converted to polar coords
			float ConeHalfAngleRad = FMath::Square(FMath::Cos(Theta) / VerticalConeHalfAngleRad) + FMath::Square(FMath::Sin(Theta) / HorizontalConeHalfAngleRad);
			ConeHalfAngleRad = FMath::Sqrt(1.f / ConeHalfAngleRad);

			// clamp to make a cone instead of a sphere
			Phi = FMath::Fmod(Phi, ConeHalfAngleRad);

			// get axes we need to rotate around
			FMatrix const DirMat = FRotationMatrix(Dir.Rotation());
			// note the axis translation, since we want the variation to be around X
			FVector const DirZ = DirMat.GetUnitAxis( EAxis::X );		
			FVector const DirY = DirMat.GetUnitAxis( EAxis::Y );

			FVector Result = Dir.RotateAngleAxis(Phi * 180.f / UE_PI, DirY);
			Result = Result.RotateAngleAxis(Theta * 180.f / UE_PI, DirZ);

			// ensure it's a unit vector (might not have been passed in that way)
			Result = Result.GetSafeNormal();

			return Result;
		}
		else
		{
			return Dir.GetSafeNormal();
		}
	}

	/**
	 * Get a textual representation of the RandomStream.
	 *
	 * @return Text describing the RandomStream.
	 */
	FString ToString() const
	{
		return FString::Printf(TEXT("FRandomStream(InitialSeed=%i, Seed=%u)"), InitialSeed, Seed);
	}

protected:

	/**
	 * Mutates the current seed into the next seed.
	 */
	void MutateSeed() const
	{
		Seed = (Seed * 196314165U) + 907633515U; 
	}

private:

	// Holds the initial seed.
	int32 InitialSeed;

	// Holds the current seed. This should be an uint32 so that any shift to obtain top bits
	// is a logical shift, rather than an arithmetic shift (which smears down the negative bit).
	mutable uint32 Seed;
};
