// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Misc/AssertionMacros.h"
#include "Misc/DateTime.h"
#include "Math/RangeBound.h"
#include "Misc/FrameNumber.h"
#include "Serialization/Archive.h"


/**
 * Template for ranges.
 *
 * Note: This class is not intended for interval arithmetic (see TInterval for that).
 *
 * A range represents a contiguous set of elements that only stores the set's
 * lower and upper bound values (aka. endpoints) for storage efficiency. Bound
 * values may be exclusive (the value is not part of the range), inclusive (the
 * value is part of the range) or open (there is no limit on the values).
 *
 * The template's primary focus is on continuous ranges, but it can be used for the
 * representation of discrete ranges as well. The element type of discrete ranges
 * has a well-defined stepping, such as an integer or a date, that separates the
 * neighboring elements. This is in contrast with continuous ranges in which the
 * step sizes, such as floats or time spans, are not of interest, and other elements
 * may be found between any two elements (although, in practice, all ranges are
 * discrete due to the limited precision of numerical values in computers).

 * When working with ranges, the user of this template is responsible for correctly
 * interpreting the range endpoints. Certain semantics will be different depending
 * on whether the range is interpreted in a continuous or discrete domain.
 *
 * Iteration of a discrete range [A, B) includes the elements A to B-1. The elements
 * of continuous ranges are generally not meant to be iterated. It is also important
 * to consider the equivalence of different representations of discrete ranges. For
 * example, the ranges [2, 6), (1, 5] and [2, 5] are equivalent in discrete domains,
 * but different in continuous ones. In order to keep this class simple, we have not
 * included canonicalization functions or auxiliary template parameters, such as
 * unit and min/max domain elements. For ease of use in most common use cases, it is
 * recommended to limit all operations to canonical ranges of the form [A, B) in
 * which the lower bound is included and the upper bound is excluded from the range.
 *
 * @param ElementType The type of elements represented by the range
 * @see RangeBound, TInterval
 */
template<typename ElementType> class TRange
{
public:

	typedef TRangeBound<ElementType> BoundsType;

	/*~ Typedef used to pass/return small element types by value rather than const& */
	typedef typename TCallTraits<ElementType>::ParamType ElementValueOrConstRef;

	/** Default constructor (no initialization). */
	[[nodiscard]] TRange() { }

	/**
	 * Create a range with a single element.
	 *
	 * The created range is of the form [A, A].
	 *
	 * @param A The element in the range.
	 */
	[[nodiscard]] explicit TRange(ElementValueOrConstRef A)
		: LowerBound(BoundsType::Inclusive(A))
		, UpperBound(BoundsType::Inclusive(A))
	{ }
	
	/**
	 * Create and initializes a new range with the given lower and upper bounds.
	 *
	 * The created range is of the form [A, B).
	 *
	 * @param A The range's lower bound value (inclusive).
	 * @param B The range's upper bound value (exclusive).
	 */
	[[nodiscard]] explicit TRange(ElementValueOrConstRef A, ElementValueOrConstRef B)
		: LowerBound(BoundsType::Inclusive(A))
		, UpperBound(BoundsType::Exclusive(B))
	{ }

	/**
	 * Create and initializes a new range with the given lower and upper bounds.
	 *
	 * @param InLowerBound The range's lower bound.
	 * @param InUpperBound The range's upper bound.
	 */
	[[nodiscard]] explicit TRange(const BoundsType& InLowerBound, const BoundsType& InUpperBound)
		: LowerBound(InLowerBound)
		, UpperBound(InUpperBound)
	{ }
	
public:

	/**
	 * Compare this range with the specified range for equality.
	 *
	 * @param Other The range to compare with.
	 * @return true if the ranges are equal, false otherwise.
	 */
	[[nodiscard]] bool operator==(const TRange& Other) const
	{
		if (IsEmpty() && Other.IsEmpty())
		{
			return true;
		}

		return ((LowerBound == Other.LowerBound) && (UpperBound == Other.UpperBound));
	}

	/**
	 * Compare this range with the specified range for inequality.
	 *
	 * @param Other The range to compare with.
	 * @return true if the ranges are not equal, false otherwise.
	 */
	[[nodiscard]] bool operator!=(const TRange& Other) const
	{
		if (IsEmpty() && Other.IsEmpty())
		{
			return false;
		}

		return ((LowerBound != Other.LowerBound) || (UpperBound != Other.UpperBound));
	}
	
public:

	/**
	 * Check whether this range adjoins to another.
	 *
	 * Two ranges are adjoint if they are next to each other without overlapping, i.e.
	 *		[A, B) and [B, C) or
	 *		[A, B] and (B, C)
	 *
	 * @param Other The other range.
	 * @return true if this range adjoins the other, false otherwise.
	 */
	[[nodiscard]] bool Adjoins(const TRange& Other) const
	{
		if (IsEmpty() || Other.IsEmpty())
		{
			return false;
		}

		if (!UpperBound.IsOpen() && !Other.LowerBound.IsOpen() && UpperBound.GetValue() == Other.LowerBound.GetValue())
		{
			return ((UpperBound.IsInclusive() && Other.LowerBound.IsExclusive()) ||
					(UpperBound.IsExclusive() && Other.LowerBound.IsInclusive()));
		}
		
		if (!Other.UpperBound.IsOpen() && !LowerBound.IsOpen() && Other.UpperBound.GetValue() == LowerBound.GetValue())
		{
			return ((Other.UpperBound.IsInclusive() && LowerBound.IsExclusive()) ||
					(Other.UpperBound.IsExclusive() && LowerBound.IsInclusive()));
		}

		return false;
	}

	/**
	 * Check whether this range conjoins the two given ranges.
	 *
	 * A range conjoins two non-overlapping ranges if it adjoins both of them, i.e.
	 *		[B, C) conjoins the two ranges [A, B) and [C, D).
	 *
	 * @param X The first range.
	 * @param Y The second range.
	 * @return true if this range conjoins the two ranges, false otherwise.
	 */
	[[nodiscard]] bool Conjoins(const TRange& X, const TRange& Y) const
	{
		if (X.Overlaps(Y))
		{
			return false;
		}

		return (Adjoins(X) && Adjoins(Y));
	}
	
	/**
	 * Check whether this range contains the specified element.
	 *
	 * @param Element The element to check.
	 * @return true if the range contains the element, false otherwise.
	 */
	[[nodiscard]] bool Contains(ElementValueOrConstRef Element) const
	{
		return ((BoundsType::MinLower(LowerBound, Element) == LowerBound) &&
				(BoundsType::MaxUpper(UpperBound, Element) == UpperBound));
	}
	
	/**
	 * Check whether this range contains another range.
	 *
	 * @param Other The range to check.
	 * @return true if the range contains the other range, false otherwise.
	 */
	[[nodiscard]] bool Contains(const TRange& Other) const
	{
		return ((BoundsType::MinLower(LowerBound, Other.LowerBound) == LowerBound) &&
				(BoundsType::MaxUpper(UpperBound, Other.UpperBound) == UpperBound));
	}

	/**
	 * Check if this range is contiguous with another range.
	 *
	 * Two ranges are contiguous if they are adjoint or overlapping.
	 *
	 * @param Other The other range.
	 * @return true if the ranges are contiguous, false otherwise.
	 */
	[[nodiscard]] bool Contiguous(const TRange& Other) const
	{
		return (Overlaps(Other) || Adjoins(Other));
	}
	
	/**
	 * Get the range's lower bound.
	 *
	 * @return Lower bound.
	 * @see GetLowerBoundValue, GetUpperBound, HasLowerBound
	 */
	[[nodiscard]] BoundsType GetLowerBound() const
	{
		return LowerBound;
	}

	/**
	 * Assign the new lower bound for this range
	 *
	 * @param NewLowerBound The new lower bound to assign
	 * @see GetLowerBound
	 */
	void SetLowerBound(const BoundsType& NewLowerBound)
	{
		LowerBound = NewLowerBound;
	}

	/**
	 * Assign the new lower bound value for this range. Current lower bound must not be open to call this method.
	 *
	 * @param NewLowerBoundValue The new lower bound value to assign
	 * @see GetLowerBound
	 */
	void SetLowerBoundValue(ElementValueOrConstRef NewLowerBoundValue)
	{
		LowerBound.SetValue(NewLowerBoundValue);
	}

	/**
	 * Get the value of the lower bound.
	 *
	 * Use HasLowerBound() to ensure that this range actually has a lower bound.
	 *
	 * @return Bound value.
	 * @see GetLowerBound, GetUpperBoundValue, HasLowerBound
	 */
	[[nodiscard]] ElementValueOrConstRef GetLowerBoundValue() const
	{
		return LowerBound.GetValue();
	}

	/**
	 * Get the range's upper bound.
	 *
	 * @return Upper bound.
	 * @see GetLowerBound, GetUpperBoundValue, HasUpperBound
	 */
	[[nodiscard]] BoundsType GetUpperBound() const
	{
		return UpperBound;
	}

	/**
	 * Assign the new upper bound for this range
	 *
	 * @param NewUpperBound The new upper bound to assign
	 * @see GetUpperBound
	 */
	void SetUpperBound(const BoundsType& NewUpperBound)
	{
		UpperBound = NewUpperBound;
	}

	/**
	 * Assign the new upper bound value for this range. Current upper bound must not be open to call this method.
	 *
	 * @param NewUpperBoundValue The new upper bound value to assign
	 * @see GetUpperBound
	 */
	void SetUpperBoundValue(ElementValueOrConstRef NewUpperBoundValue)
	{
		UpperBound.SetValue(NewUpperBoundValue);
	}

	/**
	 * Get the value of the upper bound.
	 *
	 * Use HasUpperBound() to ensure that this range actually has an upper bound.
	 *
	 * @return Bound value.
	 * @see GetLowerBoundValue, GetUpperBound, HasUpperBound
	 */
	[[nodiscard]] ElementValueOrConstRef GetUpperBoundValue() const
	{
		return UpperBound.GetValue();
	}

	/**
	 * Check whether the range has a lower bound.
	 *
	 * @return true if the range has a lower bound, false otherwise.
	 * @see GetLowerBound, GetLowerBoundValue, HasUpperBound
	 */
	[[nodiscard]] bool HasLowerBound() const
	{
		return LowerBound.IsClosed();
	}
	
	/**
	 * Check whether the range has an upper bound.
	 *
	 * @return true if the range has an upper bound, false otherwise.
	 * @see GetUpperBound, GetUpperBoundValue, HasLowerBound
	 */
	[[nodiscard]] bool HasUpperBound() const
	{
		return UpperBound.IsClosed();
	}

	/**
	 * Check whether this range is degenerate.
	 *
	 * A range is degenerate if it contains only a single element, i.e. has the following form:
	 *		[A, A]
	 *
	 * @return true if the range is degenerate, false otherwise.
	 * @see IsEmpty
	 */
	[[nodiscard]] bool IsDegenerate() const
	{
		return (LowerBound.IsInclusive() && (LowerBound == UpperBound));
	}
	
	/**
	 * Check whether this range is empty.
	 *
	 * A range is empty if it contains no elements, i.e.
	 *		(A, A)
	 *		(A, A]
	 *		[A, A)
	 *
	 * @return true if the range is empty, false otherwise.
	 * @see IsDegenerate
	 */
	[[nodiscard]] bool IsEmpty() const
	{
		if (LowerBound.IsClosed() && UpperBound.IsClosed())
		{
			if (LowerBound.GetValue() > UpperBound.GetValue())
			{
				return true;
			}

			return ((LowerBound.GetValue() == UpperBound.GetValue()) && (LowerBound.IsExclusive() || UpperBound.IsExclusive()));
		}

		return false;
	}

	/**
	 * Check whether this range overlaps with another.
	 *
	 * @param Other The other range.
	 * @return true if the ranges overlap, false otherwise.
	 */
	[[nodiscard]] bool Overlaps(const TRange& Other) const
	{
		if (IsEmpty() || Other.IsEmpty())
		{
			return false;
		}

		bool bUpperOpen = UpperBound.IsOpen() || Other.LowerBound.IsOpen();
		bool bLowerOpen = LowerBound.IsOpen() || Other.UpperBound.IsOpen();
		
		// true in the case that the bounds are open (default)
		bool bUpperValid = true;
		bool bLowerValid = true;

		if (!bUpperOpen)
		{
			bool bUpperGreaterThan = UpperBound.GetValue() > Other.LowerBound.GetValue();
			bool bUpperGreaterThanOrEqualTo = UpperBound.GetValue() >= Other.LowerBound.GetValue();
			bool bUpperBothInclusive = UpperBound.IsInclusive() && Other.LowerBound.IsInclusive();

			bUpperValid = bUpperBothInclusive ? bUpperGreaterThanOrEqualTo : bUpperGreaterThan;
		}

		if (!bLowerOpen)
		{
			bool bLowerLessThan = LowerBound.GetValue() < Other.UpperBound.GetValue();
			bool bLowerLessThanOrEqualTo = LowerBound.GetValue() <= Other.UpperBound.GetValue();
			bool bLowerBothInclusive = LowerBound.IsInclusive() && Other.UpperBound.IsInclusive();

			bLowerValid = bLowerBothInclusive ? bLowerLessThanOrEqualTo : bLowerLessThan;
		}

		return bUpperValid && bLowerValid;
	}

	/**
	 * Compute the size (diameter, length, width) of this range.
	 *
	 * The size of a closed range is the difference between its upper and lower bound values.
	 * Use IsClosed() on the lower and upper bounds before calling this method in order to
	 * make sure that the range is closed.
	 *
	 * @return Range size.
	 */
	template<typename DifferenceType>
	[[nodiscard]] DifferenceType Size() const
	{
		check(LowerBound.IsClosed() && UpperBound.IsClosed());

		return (UpperBound.GetValue() - LowerBound.GetValue());
	}
	
	/**
	 * Split the range into two ranges at the specified element.
	 *
	 * If a range [A, C) does not contain the element X, the original range is returned.
	 * Otherwise the range is split into two ranges [A, X) and [X, C), each of which may be empty.
	 *
	 * @param Element The element at which to split the range.
	 */
	[[nodiscard]] TArray<TRange> Split(ElementValueOrConstRef Element) const
	{
		TArray<TRange> Result;
		
		if (Contains(Element))
		{
			Result.Add(TRange(LowerBound, BoundsType::Exclusive(Element)));
			Result.Add(TRange(BoundsType::Inclusive(Element), UpperBound));
		}
		else
		{
			Result.Add(*this);
		}
		
		return Result;
	}

public:

	/**
	 * Calculate the difference between two ranges, i.e. X - Y.
	 *
	 * @param X The first range to subtract from.
	 * @param Y The second range to subtract with.
	 * @return Between 0 and 2 remaining ranges.
	 * @see Hull, Intersection, Union
	 */
	[[nodiscard]] static inline TArray<TRange> Difference(const TRange& X, const TRange& Y)
	{
		TArray<TRange> Result;

		if (X.Overlaps(Y))
		{
			TRange LowerRange = TRange(X.LowerBound, BoundsType::FlipInclusion(Y.LowerBound));
			TRange UpperRange = TRange(BoundsType::FlipInclusion(Y.UpperBound), X.UpperBound);
		
			if (!LowerRange.IsEmpty())
			{
				Result.Add(LowerRange);
			}

			if (!UpperRange.IsEmpty())
			{
				Result.Add(UpperRange);
			}
		}
		else
		{
			Result.Add(X);
		}

		return Result;
	}

	/**
	 * Compute the hull of two ranges.
	 *
	 * The hull is the smallest range that contains both ranges.
	 *
	 * @param X The first range.
	 * @param Y The second range.
	 * @return The hull.
	 * @see Difference, Intersection, Union
	 */
	[[nodiscard]] static inline TRange Hull(const TRange& X, const TRange& Y)
	{
		if (X.IsEmpty())
		{
			return Y;
		}

		if (Y.IsEmpty())
		{
			return X;
		}

		return TRange(BoundsType::MinLower(X.LowerBound, Y.LowerBound), BoundsType::MaxUpper(X.UpperBound, Y.UpperBound));
	}
	
	/**
	 * Compute the hull of many ranges.
	 *
	 * @param Ranges The ranges to hull.
	 * @return The hull.
	 * @see Difference, Intersection, Union
	 */
	[[nodiscard]] static inline TRange Hull(const TArray<TRange>& Ranges)
	{
		if (Ranges.Num() == 0)
		{
			return TRange::Empty();
		}

		TRange Bounds = Ranges[0];

		for (int32 i = 1; i < Ranges.Num(); ++i)
		{
			Bounds = Hull(Bounds, Ranges[i]);
		}

		return Bounds;
	}

	/**
	 * Compute the intersection of two ranges.
	 *
	 * The intersection of two ranges is the largest range that is contained by both ranges.
	 *
	 * @param X The first range.
	 * @param Y The second range.
	 * @return The intersection, or an empty range if the ranges do not overlap.
	 * @see Difference, Hull, Union
	 */
	[[nodiscard]] static inline TRange Intersection(const TRange& X, const TRange& Y)
	{
		if (X.IsEmpty())
		{
			return TRange::Empty();
		}

		if (Y.IsEmpty())
		{
			return TRange::Empty();
		}

		return TRange(BoundsType::MaxLower(X.LowerBound, Y.LowerBound), BoundsType::MinUpper(X.UpperBound, Y.UpperBound));
	}
	
	/**
	 * Compute the intersection of many ranges.
	 *
	 * @param Ranges The ranges to intersect.
	 * @return The intersection.
	 * @see Difference, Hull, Union
	 */
	[[nodiscard]] static inline TRange Intersection(const TArray<TRange>& Ranges)
	{
		if (Ranges.Num() == 0)
		{
			return TRange::Empty();
		}

		TRange Bounds = Ranges[0];

		for (int32 i = 1; i < Ranges.Num(); ++i)
		{
			Bounds = Intersection(Bounds, Ranges[i]);
		}

		return Bounds;
	}

	/**
	 * Return the union of two contiguous ranges.
	 *
	 * A union is a range or series of ranges that contains both ranges.
	 *
	 * @param X The first range.
	 * @param Y The second range.
	 * @return The union, or both ranges if the two ranges are not contiguous, or no ranges if both ranges are empty.
	 * @see Difference, Hull, Intersection
	 */
	[[nodiscard]] static inline TArray<TRange> Union(const TRange& X, const TRange& Y)
	{
		TArray<TRange> OutRanges;

		if (X.Contiguous(Y))
		{
			OutRanges.Add(TRange(BoundsType::MinLower(X.LowerBound, Y.LowerBound), BoundsType::MaxUpper(X.UpperBound, Y.UpperBound)));
		}
		else
		{
			if (!X.IsEmpty())
			{
				OutRanges.Add(X);
			}

			if (!Y.IsEmpty())
			{
				OutRanges.Add(Y);
			}
		}
		
		return OutRanges;
	}

public:

	/**
	 * Create an unbounded (open) range that contains all elements of the domain.
	 *
	 * @return A new range.
	 * @see AtLeast, AtMost, Empty, Exclusive, GreaterThan, Inclusive, LessThan
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT TRange All()
	{
		return TRange(BoundsType::Open(), BoundsType::Open());
	}

	/**
	 * Create a left-bounded range that contains all elements greater than or equal to the specified value.
	 *
	 * @param Value The value.
	 * @return A new range.
	 * @see All, AtMost, Empty, Exclusive, GreaterThan, Inclusive, LessThan
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT TRange AtLeast(ElementValueOrConstRef Value)
	{
		return TRange(BoundsType::Inclusive(Value), BoundsType::Open());
	}

	/**
	 * Create a right-bounded range that contains all elements less than or equal to the specified value.
	 *
	 * @param Value The value.
	 * @return A new range.
	 * @see All, AtLeast, Empty, Exclusive, GreaterThan, Inclusive, LessThan
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT TRange AtMost(ElementValueOrConstRef Value)
	{
		return TRange(BoundsType::Open(), BoundsType::Inclusive(Value));
	}

	/**
	 * Return an empty range.
	 *
	 * @return Empty range.
	 * @see All, AtLeast, AtMost, Exclusive, GreaterThan, Inclusive, LessThan
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT TRange Empty()
	{
		return TRange(BoundsType::Exclusive(ElementType()), BoundsType::Exclusive(ElementType()));
	}

	/**
	 * Create a range that excludes the given minimum and maximum values.
	 *
	 * @param MinThe minimum value to be included.
	 * @param Max The maximum value to be included.
	 * @return A new range.
	 * @see All, AtLeast, AtMost, Empty, Exclusive, GreaterThan, Inclusive, LessThan
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT TRange Exclusive(ElementValueOrConstRef Min, ElementValueOrConstRef Max)
	{
		return TRange(BoundsType::Exclusive(Min), BoundsType::Exclusive(Max));
	}

	/**
	 * Create a left-bounded range that contains all elements greater than the specified value.
	 *
	 * @param Value The value.
	 * @return A new range.
	 * @see All, AtLeast, AtMost, Empty, Exclusive, Inclusive, LessThan
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT TRange GreaterThan(ElementValueOrConstRef Value)
	{
		return TRange(BoundsType::Exclusive(Value), BoundsType::Open());
	}
	
	/**
	 * Create a range that includes the given minimum and maximum values.
	 *
	 * @param Min The minimum value to be included.
	 * @param Max The maximum value to be included.
	 * @return A new range.
	 * @see All, AtLeast, AtMost, Empty, Exclusive, GreaterThan, LessThan
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT TRange Inclusive(ElementValueOrConstRef Min, ElementValueOrConstRef Max)
	{
		return TRange(BoundsType::Inclusive(Min), BoundsType::Inclusive(Max));
	}

	/**
	 * Create a right-bounded range that contains all elements less than the specified value.
	 *
	 * @param Value The value.
	 * @return A new range.
	 * @see All, AtLeast, AtMost, Empty, Exclusive, GreaterThan, Inclusive
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT TRange LessThan(ElementValueOrConstRef Value)
	{
		return TRange(BoundsType::Open(), BoundsType::Exclusive(Value));
	}
	
public:

	/**
	 * Serializes the given range from or into the specified archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param Range The range to serialize.
	 * @return The archive.
	 */
	friend class FArchive& operator<<(class FArchive& Ar, TRange& Range)
	{
		return Ar << Range.LowerBound << Range.UpperBound;
	}

	/**
	 * Gets the hash for the specified range.
	 *
	 * @param Range The range to get the hash for.
	 * @return Hash value.
	 */
	friend uint32 GetTypeHash(const TRange& Range)
	{
		return (GetTypeHash(Range.LowerBound) + 23 * GetTypeHash(Range.UpperBound));
	}

private:

	/** Holds the range's lower bound. */
	BoundsType LowerBound;

	/** Holds the range's upper bound. */
	BoundsType UpperBound;
};


/* Default ranges for built-in types
 *****************************************************************************/

#define DEFINE_RANGE_WRAPPER_STRUCT(Name, ElementType) \
	struct Name : TRange<ElementType> \
	{ \
	private: \
		typedef TRange<ElementType> Super; \
	 \
	public: \
		Name() \
			: Super() \
		{ \
		} \
		 \
		Name(const Super& Rhs) \
			: Super(Rhs) \
		{ \
		} \
		 \
		explicit Name(ElementValueOrConstRef A) \
			: Super(A) \
		{ \
		} \
		 \
		explicit Name(ElementValueOrConstRef A, ElementValueOrConstRef B) \
			: Super(A, B) \
		{ \
		} \
		 \
		explicit Name(const TRangeBound<ElementType>& InLowerBound, const TRangeBound<ElementType>& InUpperBound) \
			: Super(InLowerBound, InUpperBound) \
		{ \
		} \
		 \
		TArray<Name> Split(ElementValueOrConstRef Element) const \
		{ \
			return TArray<Name>(Super::Split(Element)); \
		} \
		 \
		static UE_FORCEINLINE_HINT TArray<Name> Difference(const Name& X, const Name& Y) \
		{ \
			return TArray<Name>(Super::Difference(X, Y)); \
		} \
		 \
		static UE_FORCEINLINE_HINT Name Empty() \
		{ \
			return Super::Empty(); \
		} \
		 \
		static UE_FORCEINLINE_HINT Name Hull(const Name& X, const Name& Y) \
		{ \
			return Super::Hull(X, Y); \
		} \
		 \
		static UE_FORCEINLINE_HINT Name Hull(const TArray<Name>& Ranges) \
		{ \
			return Super::Hull(reinterpret_cast<const TArray<Super>&>(Ranges)); \
		} \
		 \
		static UE_FORCEINLINE_HINT Name Intersection(const Name& X, const Name& Y) \
		{ \
			return Super::Intersection(X, Y); \
		} \
		 \
		static UE_FORCEINLINE_HINT Name Intersection(const TArray<Name>& Ranges) \
		{ \
			return Super::Intersection(reinterpret_cast<const TArray<Super>&>(Ranges)); \
		} \
		 \
		static UE_FORCEINLINE_HINT TArray<Name> Union(const Name& X, const Name& Y) \
		{ \
			return TArray<Name>(Super::Union(X, Y)); \
		} \
		 \
		static UE_FORCEINLINE_HINT Name All() \
		{ \
			return Super::All(); \
		} \
		 \
		static UE_FORCEINLINE_HINT Name AtLeast(ElementValueOrConstRef Value) \
		{ \
			return Super::AtLeast(Value); \
		} \
		 \
		static UE_FORCEINLINE_HINT Name AtMost(ElementValueOrConstRef Value) \
		{ \
			return Super::AtMost(Value); \
		} \
		 \
		static UE_FORCEINLINE_HINT TRange GreaterThan(ElementValueOrConstRef Value) \
		{ \
			return Super::GreaterThan(Value); \
		} \
		 \
		static UE_FORCEINLINE_HINT TRange LessThan(ElementValueOrConstRef Value) \
		{ \
			return Super::LessThan(Value); \
		} \
	}; \
	 \
	template <> \
	struct TIsBitwiseConstructible<Name, TRange<ElementType>> \
	{ \
		enum { Value = true }; \
	}; \
	 \
	template <> \
	struct TIsBitwiseConstructible<TRange<ElementType>, Name> \
	{ \
		enum { Value = true }; \
	};

DEFINE_RANGE_WRAPPER_STRUCT(FDateRange,   FDateTime)
DEFINE_RANGE_WRAPPER_STRUCT(FDoubleRange, double)
DEFINE_RANGE_WRAPPER_STRUCT(FFloatRange,  float)
DEFINE_RANGE_WRAPPER_STRUCT(FInt8Range,   int8)
DEFINE_RANGE_WRAPPER_STRUCT(FInt16Range,  int16)
DEFINE_RANGE_WRAPPER_STRUCT(FInt32Range,  int32)
DEFINE_RANGE_WRAPPER_STRUCT(FInt64Range,  int64)
DEFINE_RANGE_WRAPPER_STRUCT(FFrameNumberRange, FFrameNumber)
