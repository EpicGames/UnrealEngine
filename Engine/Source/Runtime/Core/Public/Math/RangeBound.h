// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Templates/TypeHash.h"
#include "Containers/EnumAsByte.h"
#include "Misc/FrameNumber.h"
#include "Misc/DateTime.h"
#include "UObject/CoreReflectedTypeAccessors.h"

namespace ERangeBoundTypes
{
	/**
	 * Enumerates the valid types of range bounds.
	 */
	enum Type : int
	{
		/** The range excludes the bound. */
		Exclusive,

		/** The range includes the bound. */
		Inclusive,

		/** The bound is open. */
		Open
	};
}

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<ERangeBoundTypes::Type>();
#endif


/**
 * Template for range bounds.
 */
template<typename ElementType>
class TRangeBound
{
public:

	/*~ Typedef used to pass/return small element types by value rather than const& */
	typedef typename TCallTraits<ElementType>::ParamType ElementValueOrConstRef;

	/**
	 * Default constructor.
	 *
	 * @see Exclusive, Inclusive, Open
	 */
	[[nodiscard]] TRangeBound()
		: Type(ERangeBoundTypes::Open)
		, Value()
	{ }

	/**
	 * Creates a closed bound that includes the specified value.
	 *
	 * @param InValue The bound's value.
	 * @see Exclusive, Inclusive, Open
	 */
	[[nodiscard]] TRangeBound(ElementValueOrConstRef InValue)
		: Type(ERangeBoundTypes::Inclusive)
		, Value(InValue)
	{ }

public:

	/**
	 * Compares this bound with the specified bound for equality.
	 *
	 * @param Other The bound to compare with.
	 * @return true if the bounds are equal, false otherwise.
	 */
	[[nodiscard]] bool operator==(const TRangeBound& Other) const
	{
		return ((Type == Other.Type) && (IsOpen() || (Value == Other.Value)));
	}

	/**
	 * Compares this range with the specified bound for inequality.
	 *
	 * @param Other The bound to compare with.
	 * @return true if the bounds are not equal, false otherwise.
	 */
	[[nodiscard]] bool operator!=(const TRangeBound& Other) const
	{
		return ((Type != Other.Type) || (!IsOpen() && (Value != Other.Value)));
	}

public:

	/**
	 * Gets the bound's value.
	 *
	 * Use IsClosed() to verify that this bound is closed before calling this method.
	 *
	 * @return Bound value.
	 * @see IsOpen
	 */
	[[nodiscard]] inline ElementValueOrConstRef GetValue() const
	{
		check(Type != ERangeBoundTypes::Open);

		return Value;
	}


	/**
	 * Sets the bound's value, maintining the inclusivity of the bound
	 *
	 * Use IsClosed() to verify that this bound is closed before calling this method.
	 *
	 * @param NewValue New bound value.
	 * @see IsClosed
	 */
	inline void SetValue(ElementValueOrConstRef NewValue)
	{
		check(Type != ERangeBoundTypes::Open);

		Value = NewValue;
	}

	/**
	 * Checks whether the bound is closed.
	 *
	 * @return true if the bound is closed, false otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsClosed() const
	{
		return (Type != ERangeBoundTypes::Open);
	}

	/**
	 * Checks whether the bound is exclusive.
	 *
	 * @return true if the bound is exclusive, false otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsExclusive() const
	{
		return (Type == ERangeBoundTypes::Exclusive);
	}

	/**
	 * Checks whether the bound is inclusive.
	 *
	 * @return true if the bound is inclusive, false otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsInclusive() const
	{
		return (Type == ERangeBoundTypes::Inclusive);
	}

	/**
	 * Checks whether the bound is open.
	 *
	 * @return true if the bound is open, false otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsOpen() const
	{
		return (Type == ERangeBoundTypes::Open);
	}

public:

	/**
	 * Serializes the given bound from or into the specified archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param Bound The bound to serialize.
	 * @return The archive.
	 */
	friend class FArchive& operator<<(class FArchive& Ar, TRangeBound& Bound)
	{
		return Ar << (uint8&)Bound.Type << Bound.Value;
	}

	/**
	 * Gets the hash for the specified bound.
	 *
	 * @param Bound The bound to get the hash for.
	 * @return Hash value.
	 */
	friend uint32 GetTypeHash(const TRangeBound& Bound)
	{
		return (GetTypeHash((uint8)Bound.Type) + 23 * GetTypeHash(Bound.Value));
	}

public:

	/**
	 * Returns a closed bound that excludes the specified value.
	 *
	 * @param Value The bound value.
	 * @return An exclusive closed bound.
	 */
	[[nodiscard]] static inline TRangeBound Exclusive(ElementValueOrConstRef Value)
	{
		TRangeBound Result;

		Result.Type = ERangeBoundTypes::Exclusive;
		Result.Value = Value;

		return Result;
	}

	/**
	 * Returns a closed bound that includes the specified value.
	 *
	 * @param Value The bound value.
	 * @return An inclusive closed bound.
	 */
	[[nodiscard]] static inline TRangeBound Inclusive(ElementValueOrConstRef Value)
	{
		TRangeBound Result;

		Result.Type = ERangeBoundTypes::Inclusive;
		Result.Value = Value;

		return Result;
	}

	/**
	 * Returns an open bound.
	 *
	 * @return An open bound.
	 */
	[[nodiscard]] static inline TRangeBound Open()
	{
		TRangeBound Result;

		Result.Type = ERangeBoundTypes::Open;

		return Result;
	}

public:

	/**
	 * Returns the given bound with its inclusion flipped between inclusive and exclusive.
	 *
	 * If the bound is open it is returned unchanged.
	 *
	 * @return A new bound.
	 */
	[[nodiscard]] static inline TRangeBound FlipInclusion(const TRangeBound& Bound)
	{
		if (Bound.IsExclusive())
		{
			return Inclusive(Bound.Value);
		}

		if (Bound.IsInclusive())
		{
			return Exclusive(Bound.Value);
		}

		return Bound;
	}

	/**
	 * Returns the greater of two lower bounds.
	 *
	 * @param A The first lower bound.
	 * @param B The second lower bound.
	 * @return The greater lower bound.
	 */
	[[nodiscard]] static inline const TRangeBound& MaxLower(const TRangeBound& A, const TRangeBound& B)
	{
		if (A.IsOpen()) { return B; }
		if (B.IsOpen()) { return A; }
		if (A.Value > B.Value) { return A; }
		if (B.Value > A.Value) { return B; }
		if (A.IsExclusive()) { return A; }

		return B;
	}

	/**
	 * Returns the greater of two upper bounds.
	 *
	 * @param A The first upper bound.
	 * @param B The second upper bound.
	 * @return The greater upper bound.
	 */
	[[nodiscard]] static inline const TRangeBound& MaxUpper(const TRangeBound& A, const TRangeBound& B)
	{
		if (A.IsOpen()) { return A; }
		if (B.IsOpen()) { return B; }
		if (A.Value > B.Value) { return A; }
		if (B.Value > A.Value) { return B; }
		if (A.IsInclusive()) { return A; }

		return B;
	}

	/**
	 * Returns the lesser of two lower bounds.
	 *
	 * @param A The first lower bound.
	 * @param B The second lower bound.
	 * @return The lesser lower bound.
	 */
	[[nodiscard]] static inline const TRangeBound& MinLower(const TRangeBound& A, const TRangeBound& B)
	{
		if (A.IsOpen()) { return A; }
		if (B.IsOpen()) { return B; }
		if (A.Value < B.Value) { return A; }
		if (B.Value < A.Value) { return B; }
		if (A.IsInclusive()) { return A; }

		return B;
	}

	/**
	 * Returns the lesser of two upper bounds.
	 *
	 * @param A The first upper bound.
	 * @param B The second upper bound.
	 * @return The lesser upper bound.
	 */
	[[nodiscard]] static inline const TRangeBound& MinUpper(const TRangeBound& A, const TRangeBound& B)
	{
		if (A.IsOpen()) { return B; }
		if (B.IsOpen()) { return A; }
		if (A.Value < B.Value) { return A; }
		if (B.Value < A.Value) { return B; }
		if (A.IsExclusive()) { return A; }

		return B;
	}

private:

	/** Holds the type of the bound. */
	TEnumAsByte<ERangeBoundTypes::Type> Type;

	/** Holds the bound's value. */
	ElementType Value;
};


/* Default range bounds for built-in types (for FProperty support)
 *****************************************************************************/

#define DEFINE_RANGEBOUND_WRAPPER_STRUCT(Name, ElementType) \
	struct Name : TRangeBound<ElementType> \
	{ \
	private: \
		typedef TRangeBound<ElementType> Super; \
	 \
	public: \
		Name() \
			: Super() \
		{ } \
		 \
		Name(const Super& Other) \
			: Super(Other) \
		{ } \
		 \
		Name(const ElementType InValue) \
			: Super(InValue) \
		{ } \
		 \
		static UE_FORCEINLINE_HINT Name Exclusive(ElementValueOrConstRef Value) \
		{ \
			return static_cast<const Name&>(Super::Exclusive(Value)); \
		} \
		 \
		static UE_FORCEINLINE_HINT Name Inclusive(ElementValueOrConstRef Value) \
		{ \
			return static_cast<const Name&>(Super::Inclusive(Value)); \
		} \
		 \
		static UE_FORCEINLINE_HINT Name Open() \
		{ \
			return static_cast<const Name&>(Super::Open()); \
		} \
		 \
		static UE_FORCEINLINE_HINT Name FlipInclusion(const Name& Bound) \
		{ \
			return static_cast<const Name&>(Super::FlipInclusion(Bound)); \
		} \
		 \
		static UE_FORCEINLINE_HINT const Name& MaxLower(const Name& A, const Name& B) \
		{ \
			return static_cast<const Name&>(Super::MaxLower(A, B)); \
		} \
		 \
		static UE_FORCEINLINE_HINT const Name& MaxUpper(const Name& A, const Name& B) \
		{ \
			return static_cast<const Name&>(Super::MaxUpper(A, B)); \
		} \
		 \
		static UE_FORCEINLINE_HINT const Name& MinLower(const Name& A, const Name& B) \
		{ \
			return static_cast<const Name&>(Super::MinLower(A, B)); \
		} \
		 \
		static UE_FORCEINLINE_HINT const Name& MinUpper(const Name& A, const Name& B) \
		{ \
			return static_cast<const Name&>(Super::MinUpper(A, B)); \
		} \
	}; \
	 \
	template <> \
	struct TIsBitwiseConstructible<Name, TRangeBound<ElementType>> \
	{ \
		enum { Value = true }; \
	}; \
	 \
	template <> \
	struct TIsBitwiseConstructible<TRangeBound<ElementType>, Name> \
	{ \
		enum { Value = true }; \
	};


DEFINE_RANGEBOUND_WRAPPER_STRUCT(FDateRangeBound,   FDateTime)
DEFINE_RANGEBOUND_WRAPPER_STRUCT(FDoubleRangeBound, double)
DEFINE_RANGEBOUND_WRAPPER_STRUCT(FFloatRangeBound,  float)
DEFINE_RANGEBOUND_WRAPPER_STRUCT(FInt8RangeBound,   int8)
DEFINE_RANGEBOUND_WRAPPER_STRUCT(FInt16RangeBound,  int16)
DEFINE_RANGEBOUND_WRAPPER_STRUCT(FInt32RangeBound,  int32)
DEFINE_RANGEBOUND_WRAPPER_STRUCT(FInt64RangeBound,  int64)
DEFINE_RANGEBOUND_WRAPPER_STRUCT(FFrameNumberRangeBound, FFrameNumber)
