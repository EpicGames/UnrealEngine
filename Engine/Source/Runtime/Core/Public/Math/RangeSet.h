// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Math/Range.h"
#include "Serialization/Archive.h"


/**
 * Template for range sets.
 *
 * @todo gmp: Implement more efficient storage of range sets
 */
template<typename ElementType> class TRangeSet
{
	typedef TRangeBound<ElementType> BoundsType;
	typedef TRange<ElementType> RangeType;

	/*~ Typedef used to pass/return small element types by value rather than const& */
	typedef typename TCallTraits<ElementType>::ParamType ElementValueOrConstRef;

public:

	/** Default constructor. */
	[[nodiscard]] TRangeSet() { }

	/** Destructor. */
	~TRangeSet() = default;

public:
	
	/**
	 * Adds a range to the set.
	 *
	 * This method merges overlapping ranges into a single range (i.e. {[1, 5], [4, 6]} becomes [1, 6]).
	 * Adjacent ranges (i.e. {[1, 4), [4, 6)} are also merged.
	 *
	 * @param Range The range to add.
	 */
	void Add(RangeType Range)
	{
		for (int32 Index = 0; Index < Ranges.Num(); ++Index)
		{
			const RangeType& Current = Ranges[Index];

			if (Current.Adjoins(Range) || Current.Overlaps(Range))
			{
				Range = RangeType(
					BoundsType::MinLower(Current.GetLowerBound(), Range.GetLowerBound()),
					BoundsType::MaxUpper(Current.GetUpperBound(), Range.GetUpperBound())
				);

				Ranges.RemoveAtSwap(Index--);
			}
		}

		Ranges.Add(Range);
	}

	/**
	 * Merges another range set into this set.
	 *
	 * @param Other The range set to merge.
	 */
	void Merge(const TRangeSet& Other)
	{
		for (typename TArray<RangeType>::TConstIterator It(Other.Ranges); It; ++It)
		{
			Add(*It);
		}
	}

	/**
	 * Removes a range from the set.
	 *
	 * Ranges that overlap with the removed range will be split.
	 *
	 * @param Range The range to remove.
	 *//*
	void Remove(const RangeType& Range)
	{
		for (int32 Index = 0; Index < Ranges.Num(); ++Index)
		{
			const RangeType& Current = Ranges(Index);

			if (Current.Overlaps(Range))
			{
				if (Current.GetLowerBound() < Range.GetLowerBound())
				{
					Ranges.Add(RangeType(Current.GetLowerBound(), Range.GetLowerBound()));
				}

				if (Current.GetUpperBound() > Range.GetUpperBound())
				{
					Ranges.Add(RangeType(Range.GetUpperBound(), Current.GetUpperBound()));
				}

				Ranges.RemoveAtSwap(Index--);
			}
		}
	}*/

	/** Removes all ranges from the set. */
	void Empty()
	{
		Ranges.Empty();
	}
	
public:
	
	/**
	 * Checks whether this set contains the specified element.
	 *
	 * @param Element The element to check.
	 * @return true if the element is in the set, false otherwise.
	 */
	[[nodiscard]] bool Contains(ElementValueOrConstRef Element) const
	{
		for (typename TArray<RangeType>::TConstIterator It(Ranges); It; ++It)
		{
			if (It->Contains(Element))
			{
				return true;
			}
		}

		return false;
	}
	
	/**
	 * Checks whether this set contains the specified range.
	 *
	 * @param Range The range to check.
	 * @return true if the set contains the range, false otherwise.
	 */
	[[nodiscard]] bool Contains(const RangeType& Range) const
	{
		for (typename TArray<RangeType>::TConstIterator It(Ranges); It; ++It)
		{
			if (It->Contains(Range))
			{
				return true;
			}
		}

		return false;
	}

	/**
	 * Gets the range set's lowest bound.
	 *
	 * @return Lowest bound.
	 * @see GetMaxBound, GetMinBoundValue, HasMinBound
	 */
	[[nodiscard]] BoundsType GetMinBound() const
	{
		BoundsType Result;

		if (Ranges.Num())
		{
			Result = Ranges[0].GetLowerBound();
			for (int32 i = 1; i < Ranges.Num(); i++)
			{
				Result = BoundsType::MinLower(Result, Ranges[i].GetLowerBound());
			}
		}

		return Result;
	}

	/**
	 * Gets the value of the lowest bound.
	 *
	 * Use HasMinBound() to ensure that this range set actually has a lowest bound.
	 *
	 * @return Bound value.
	 * @see GetMaxBoundValue, GetMinBound, HasMinBound
	 */
	[[nodiscard]] ElementType GetMinBoundValue() const
	{
		return GetMinBound().GetValue();
	}

	/**
	 * Gets the range set's uppermost bound.
	 *
	 * @return Uppermost bound.
	 * @see GetMaxBoundValue, GetMinBound, HasMaxBound
	 */
	[[nodiscard]] BoundsType GetMaxBound() const
	{
		BoundsType Result;

		if (Ranges.Num())
		{
			Result = Ranges[0].GetUpperBound();
			for (int32 i = 1; i < Ranges.Num(); i++)
			{
				Result = BoundsType::MaxUpper(Result, Ranges[i].GetUpperBound());
			}
		}

		return Result;
	}

	/**
	 * Gets the value of the uppermost bound.
	 *
	 * Use HasMaxBound() to ensure that this range actually has an upper bound.
	 *
	 * @return Bound value.
	 * @see GetMaxBound, GetMinBoundValue, HasMaxBound
	 */
	[[nodiscard]] ElementType GetMaxBoundValue() const
	{
		return GetMaxBound().GetValue();
	}


	/**
	 * Returns a read-only collection of the ranges contained in this set.
	 *
	 * @param Allocator The array allocator to use.
	 * @param OutRanges Will contain the collection of ranges.
	 */
	template<typename Allocator>
	const void GetRanges(TArray<RangeType, Allocator>& OutRanges) const
	{
		OutRanges = Ranges;
	}
	
	/**
	 * Checks whether the range has a lowest bound.
	 *
	 * @return true if the range has a lowest bound, false otherwise.
	 * @see GetMinBound, GetMinBoundValue, HasMaxBound
	 */
	[[nodiscard]] bool HasMinBound() const
	{
		return GetMinBound().IsClosed();
	}
	
	/**
	 * Checks whether the range has an uppermost bound.
	 *
	 * @return true if the range has an uppermost bound, false otherwise.
	 * @see GetUpperBound, GetUpperBoundValue, HasMinBound
	 */
	[[nodiscard]] bool HasMaxBound() const
	{
		return GetMaxBound().IsClosed();
	}

	/**
	 * Checks whether this range set is empty.
	 *
	 * @return true if the range set is empty, false otherwise.
	 */
	[[nodiscard]] bool IsEmpty() const
	{
		return (Ranges.Num() == 0);
	}
	
	/**
	 * Checks whether this range set overlaps with the specified range.
	 *
	 * @param Range The range to check.
	 * @return true if this set overlaps with the range, false otherwise.
	 */
	[[nodiscard]] bool Overlaps(const RangeType& Range) const
	{
		for (typename TArray<RangeType>::TConstIterator It(Ranges); It; ++It)
		{
			if (It->Overlaps(Range))
			{
				return true;
			}
		}

		return false;
	}
	
	/**
	 * Checks whether this range set overlaps with another.
	 *
	 * @param Other The other range set.
	 * @return true if the range sets overlap, false otherwise.
	 *
	 * @todo gmp: This could be optimized to O(n*logn) using a line sweep on a pre-sorted array of bounds.
	 */
	[[nodiscard]] bool Overlaps(const TRangeSet& Other) const
	{
		for (typename TArray<RangeType>::TConstIterator It(Other.Ranges); It; ++It)
		{
			if (Overlaps(*It))
			{
				return true;
			}
		}

		return false;
	}

public:

	/**
	 * Serializes the given range set from or into the specified archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param RangeSet The range set to serialize.
	 * @return The archive.
	 */
	friend class FArchive& operator<<(class FArchive& Ar, TRangeSet& RangeSet)
	{
		return Ar << RangeSet.Ranges;
	}

private:

	/** Holds the set of ranges. */
	TArray<RangeType> Ranges;
};
