// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h" // IWYU pragma: export
#include "Math/Vector.h"


/**
 * 3D Ray represented by Origin and (normalized) Direction
 */
namespace UE
{
namespace Math
{

template<typename T>
struct TRay
{
public:
	using FReal = T;

	/** Ray origin point */
	TVector<T> Origin;

	/** Ray direction vector (always normalized) */
	TVector<T> Direction;

public:

	/** Default constructor initializes ray to Zero origin and Z-axis direction */
	[[nodiscard]] TRay()
	{
		Init();
	}

	/**
	 * Creates and initializes a new Ray to Zero Origin and Z-axis Direction.
	 * @param EForceInit Force Init Enum.
	 */
	[[nodiscard]] explicit TRay<T>( EForceInit )
	{
		Init();
	}


	/** 
	  * Initialize Ray with origin and direction
	  *
	  * @param Origin Ray Origin Point
	  * @param Direction Ray Direction Vector
	  * @param bDirectionIsNormalized Direction will be normalized unless this is passed as true (default false)
	  */
	[[nodiscard]] TRay(const TVector<T>& Origin, const TVector<T>& Direction, bool bDirectionIsNormalized = false)
	{
		this->Origin = Origin;
		this->Direction = Direction;
		if (bDirectionIsNormalized == false)
		{
			this->Direction.Normalize();    // is this a full-accuracy sqrt?
		}
	}


	/**
	 * Set the initial values of the Ray to Zero Origin and Z-axis Direction.
	 */
	void Init()
	{
		Origin = TVector<T>::ZeroVector;
		Direction = TVector<T>(0, 0, 1);
	}

	/**
	 * Compares two Rays for equality.
	 *
	 * @param Other The other Ray to compare with.
	 * @return true if the Rays are equal, false otherwise.
	 */
	[[nodiscard]] bool operator==( const TRay<T>& Other ) const
	{
		return (Origin == Other.Origin) && (Direction == Other.Direction);
	}

	/**
	 * Compares two Rays for inequality.
	 *
	 * @param Other The other Ray to compare with.
	 * @return true if the Rays are not equal, false otherwise.
	 */
	[[nodiscard]] bool operator!=( const TRay<T>& Other ) const
	{
		return !(*this == Other);
	}


public:

	/** 
	 * Calculate position on ray at given distance/parameter
	 *
	 * @param RayParameter Scalar distance along Ray
	 * @return Point on Ray
	 */
	[[nodiscard]] TVector<T> PointAt(T RayParameter) const
	{
		return Origin + RayParameter * Direction;
	}

	/**
	 * Calculate ray parameter (distance from origin to closest point) for query Point
	 *
	 * @param Point query Point
	 * @return distance along ray from origin to closest point
	 */
	[[nodiscard]] T GetParameter(const TVector<T>& Point) const
	{
		return TVector<T>::DotProduct((Point - Origin), Direction);
	}

	/**
	 * Find minimum squared distance from query point to ray
	 *
	 * @param Point query Point
	 * @return squared distance to Ray
	 */
	[[nodiscard]] T DistSquared(const TVector<T>& Point) const
	{
		T RayParameter = TVector<T>::DotProduct((Point - Origin), Direction);
		if (RayParameter < 0)
		{
			return TVector<T>::DistSquared(Origin, Point);
		}
		else 
		{
			TVector<T> ProjectionPt = Origin + RayParameter * Direction;
			return TVector<T>::DistSquared(ProjectionPt, Point);
		}
	}

	/**
	 * Find minimum distance from query point to ray
	 *
	 * @param Point query Point
	 * @return distance to Ray
	 */
	[[nodiscard]] T Dist(const TVector<T>& Point) const
	{
		return FMath::Sqrt(DistSquared(Point));
	}

	/**
	 * Find closest point on ray to query point
	 * @param Point query point
	 * @return closest point on Ray
	 */
	[[nodiscard]] TVector<T> ClosestPoint(const TVector<T>& Point) const
	{
		T RayParameter = TVector<T>::DotProduct((Point - Origin), Direction);
		if (RayParameter < 0) 
		{
			return Origin;
		}
		else 
		{
			return Origin + RayParameter * Direction;
		}
	}



public:

	/**
	* Get a textual representation of the Ray.
	*
	* @return Text describing the Ray.
	*/
	[[nodiscard]] FString ToString() const
	{
		return FString::Printf(TEXT("Origin=(%s), Direction=(%s)"), *Origin.ToString(), *Direction.ToString());
	}

	/**
	 * Serializes the Ray.
	 *
	 * @param Ar The archive to serialize into.
	 * @param Box The box to serialize.
	 *
	 * @return Reference to the Archive after serialization.
	 */
	friend FArchive& operator<<( FArchive& Ar, TRay<T>& Ray )
	{
		return Ar << Ray.Origin << Ray.Direction;
	}

	// Note: TRay is usually written via binary serialization. This function exists for SerializeFromMismatchedTag conversion usage. 
	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		if constexpr (std::is_same_v<T, float>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Ray, Ray3f, Ray3d);
		}
		else if constexpr (std::is_same_v<T, double>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Ray, Ray3d, Ray3f);
		}
		else
		{
			static_assert(sizeof(T) == 0, "Unimplemented");
			return false;
		}
	}

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TRay(const TRay<FArg>& From)
		: TRay<T>(TVector<T>(From.Origin), TVector<T>(From.Direction), true)
	{
	}
};

}	// namespace UE::Math
}	// namespace UE

UE_DECLARE_LWC_TYPE(Ray, 3);

template<> struct TIsPODType<FRay3f> { enum { Value = true }; };
template<> struct TIsPODType<FRay3d> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FRay3f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FRay3d> { enum { Value = true }; };
template<> struct TCanBulkSerialize<FRay3f> { enum { Value = true }; };
template<> struct TCanBulkSerialize<FRay3d> { enum { Value = true }; };
