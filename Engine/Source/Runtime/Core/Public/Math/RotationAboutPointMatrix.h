// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector.h"
#include "Math/Matrix.h"
#include "Math/Quat.h"
#include "Math/RotationTranslationMatrix.h"

namespace UE {
namespace Math {

/** Rotates about an Origin point. */
template<typename T>	
struct TRotationAboutPointMatrix
	: public TRotationTranslationMatrix<T>
{
public:
	using TRotationTranslationMatrix<T>::M;

	/**
	 * Constructor.
	 *
	 * @param Rot rotation
	 * @param Origin about which to rotate.
	 */
	[[nodiscard]] TRotationAboutPointMatrix(const TRotator<T>& Rot, const TVector<T>& Origin);

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	[[nodiscard]] explicit TRotationAboutPointMatrix(const TRotationAboutPointMatrix<FArg>& From)
		: TRotationTranslationMatrix<T>(From)
	{
	}

	/** Matrix factory. Return an TMatrix<T> so we don't have type conversion issues in expressions. */
	[[nodiscard]] static TMatrix<T> Make(const TRotator<T>& Rot, const TVector<T>& Origin)
	{
		return TRotationAboutPointMatrix(Rot, Origin);
	}

	/** Matrix factory. Return an TMatrix<T> so we don't have type conversion issues in expressions. */
	[[nodiscard]] static TMatrix<T> Make(const TQuat<T>& Rot, const TVector<T>& Origin)
	{
		return TRotationAboutPointMatrix(Rot.Rotator(), Origin);
	}
};

template<typename T>
inline TRotationAboutPointMatrix<T>::TRotationAboutPointMatrix(const TRotator<T>& Rot, const TVector<T>& Origin)
	: TRotationTranslationMatrix<T>(Rot, Origin)
{
	// FRotationTranslationMatrix generates R * T.
	// We need -T * R * T, so prepend that translation:
	TVector<T> XAxis(M[0][0], M[1][0], M[2][0]);
	TVector<T> YAxis(M[0][1], M[1][1], M[2][1]);
	TVector<T> ZAxis(M[0][2], M[1][2], M[2][2]);

	M[3][0]	-= XAxis | Origin;
	M[3][1]	-= YAxis | Origin;
	M[3][2]	-= ZAxis | Origin;
}

} // namespace Math
} // namespace UE

UE_DECLARE_LWC_TYPE(RotationAboutPointMatrix, 44);

template<> struct TIsUECoreVariant<FRotationAboutPointMatrix44f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FRotationAboutPointMatrix44d> { enum { Value = true }; };