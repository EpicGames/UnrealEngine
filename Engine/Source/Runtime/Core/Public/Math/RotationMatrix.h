// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector.h"
#include "Math/Rotator.h"
#include "Math/Matrix.h"
#include "Math/RotationTranslationMatrix.h"
#include "Math/QuatRotationTranslationMatrix.h"

namespace UE {
namespace Math {

/** Rotation matrix no translation */
template<typename T>
struct TRotationMatrix
	: public TRotationTranslationMatrix<T>
{
public:
	using TRotationTranslationMatrix<T>::M;

	/**
	 * Constructor.
	 *
	 * @param Rot rotation
	 */
	[[nodiscard]] TRotationMatrix(const TRotator<T>& Rot)
		: TRotationTranslationMatrix<T>(Rot, TVector<T>::ZeroVector)
	{ }

	/** Matrix factory. Return an TMatrix<T> so we don't have type conversion issues in expressions. */
	[[nodiscard]] static TMatrix<T> Make(TRotator<T> const& Rot)
	{
		return TRotationMatrix(Rot);
	}

	/** Matrix factory. Return an TMatrix<T> so we don't have type conversion issues in expressions. */
	[[nodiscard]] static TMatrix<T> Make(TQuat<T> const& Rot);

	/** Builds a rotation matrix given only a XAxis. Y and Z are unspecified but will be orthonormal. XAxis need not be normalized. */
	[[nodiscard]] static TMatrix<T> MakeFromX(TVector<T> const& XAxis);

	/** Builds a rotation matrix given only a YAxis. X and Z are unspecified but will be orthonormal. YAxis need not be normalized. */
	[[nodiscard]] static TMatrix<T> MakeFromY(TVector<T> const& YAxis);

	/** Builds a rotation matrix given only a ZAxis. X and Y are unspecified but will be orthonormal. ZAxis need not be normalized. */
	[[nodiscard]] static TMatrix<T> MakeFromZ(TVector<T> const& ZAxis);

	/** Builds a matrix with given X and Y axes. X will remain fixed, Y may be changed minimally to enforce orthogonality. Z will be computed. Inputs need not be normalized. */
	[[nodiscard]] static TMatrix<T> MakeFromXY(TVector<T> const& XAxis, TVector<T> const& YAxis);

	/** Builds a matrix with given X and Z axes. X will remain fixed, Z may be changed minimally to enforce orthogonality. Y will be computed. Inputs need not be normalized. */
	[[nodiscard]] static TMatrix<T> MakeFromXZ(TVector<T> const& XAxis, TVector<T> const& ZAxis);

	/** Builds a matrix with given Y and X axes. Y will remain fixed, X may be changed minimally to enforce orthogonality. Z will be computed. Inputs need not be normalized. */
	[[nodiscard]] static TMatrix<T> MakeFromYX(TVector<T> const& YAxis, TVector<T> const& XAxis);

	/** Builds a matrix with given Y and Z axes. Y will remain fixed, Z may be changed minimally to enforce orthogonality. X will be computed. Inputs need not be normalized. */
	[[nodiscard]] static TMatrix<T> MakeFromYZ(TVector<T> const& YAxis, TVector<T> const& ZAxis);

	/** Builds a matrix with given Z and X axes. Z will remain fixed, X may be changed minimally to enforce orthogonality. Y will be computed. Inputs need not be normalized. */
	[[nodiscard]] static TMatrix<T> MakeFromZX(TVector<T> const& ZAxis, TVector<T> const& XAxis);

	/** Builds a matrix with given Z and Y axes. Z will remain fixed, Y may be changed minimally to enforce orthogonality. X will be computed. Inputs need not be normalized. */
	[[nodiscard]] static TMatrix<T> MakeFromZY(TVector<T> const& ZAxis, TVector<T> const& YAxis);
};

template<typename T>
TMatrix<T> TRotationMatrix<T>::Make(TQuat<T> const& Rot)
{
	return TQuatRotationTranslationMatrix<T>(Rot, TVector<T>::ZeroVector);
}

template<typename T>
TMatrix<T> TRotationMatrix<T>::MakeFromX(TVector<T> const& XAxis)
{
	TVector<T> const NewX = XAxis.GetSafeNormal();

	// try to use up if possible
	TVector<T> const UpVector = (FMath::Abs(NewX.Z) < (1.f - UE_KINDA_SMALL_NUMBER)) ? TVector<T>(0, 0, 1.f) : TVector<T>(1.f, 0, 0);

	const TVector<T> NewY = (UpVector ^ NewX).GetSafeNormal();
	const TVector<T> NewZ = NewX ^ NewY;

	return TMatrix<T>(NewX, NewY, NewZ, TVector<T>::ZeroVector);
}

template<typename T>
TMatrix<T> TRotationMatrix<T>::MakeFromY(TVector<T> const& YAxis)
{
	TVector<T> const NewY = YAxis.GetSafeNormal();

	// try to use up if possible
	TVector<T> const UpVector = (FMath::Abs(NewY.Z) < (1.f - UE_KINDA_SMALL_NUMBER)) ? TVector<T>(0, 0, 1.f) : TVector<T>(1.f, 0, 0);

	const TVector<T> NewZ = (UpVector ^ NewY).GetSafeNormal();
	const TVector<T> NewX = NewY ^ NewZ;

	return TMatrix<T>(NewX, NewY, NewZ, TVector<T>::ZeroVector);
}

template<typename T>
TMatrix<T> TRotationMatrix<T>::MakeFromZ(TVector<T> const& ZAxis)
{
	TVector<T> const NewZ = ZAxis.GetSafeNormal();

	// try to use up if possible
	TVector<T> const UpVector = (FMath::Abs(NewZ.Z) < (1.f - UE_KINDA_SMALL_NUMBER)) ? TVector<T>(0, 0, 1.f) : TVector<T>(1.f, 0, 0);

	const TVector<T> NewX = (UpVector ^ NewZ).GetSafeNormal();
	const TVector<T> NewY = NewZ ^ NewX;

	return TMatrix<T>(NewX, NewY, NewZ, TVector<T>::ZeroVector);
}

template<typename T>
TMatrix<T> TRotationMatrix<T>::MakeFromXY(TVector<T> const& XAxis, TVector<T> const& YAxis)
{
	TVector<T> NewX = XAxis.GetSafeNormal();
	TVector<T> Norm = YAxis.GetSafeNormal();

	// if they're almost same, we need to find arbitrary vector
	if (FMath::IsNearlyEqual(FMath::Abs(NewX | Norm), T(1.f)))
	{
		// make sure we don't ever pick the same as NewX
		Norm = (FMath::Abs(NewX.Z) < (1.f - UE_KINDA_SMALL_NUMBER)) ? TVector<T>(0, 0, 1.f) : TVector<T>(1.f, 0, 0);
	}

	const TVector<T> NewZ = (NewX ^ Norm).GetSafeNormal();
	const TVector<T> NewY = NewZ ^ NewX;

	return TMatrix<T>(NewX, NewY, NewZ, TVector<T>::ZeroVector);
}

template<typename T>
TMatrix<T> TRotationMatrix<T>::MakeFromXZ(TVector<T> const& XAxis, TVector<T> const& ZAxis)
{
	TVector<T> const NewX = XAxis.GetSafeNormal();
	TVector<T> Norm = ZAxis.GetSafeNormal();

	// if they're almost same, we need to find arbitrary vector
	if (FMath::IsNearlyEqual(FMath::Abs(NewX | Norm), T(1.f)))
	{
		// make sure we don't ever pick the same as NewX
		Norm = (FMath::Abs(NewX.Z) < (1.f - UE_KINDA_SMALL_NUMBER)) ? TVector<T>(0, 0, 1.f) : TVector<T>(1.f, 0, 0);
	}

	const TVector<T> NewY = (Norm ^ NewX).GetSafeNormal();
	const TVector<T> NewZ = NewX ^ NewY;

	return TMatrix<T>(NewX, NewY, NewZ, TVector<T>::ZeroVector);
}

template<typename T>
TMatrix<T> TRotationMatrix<T>::MakeFromYX(TVector<T> const& YAxis, TVector<T> const& XAxis)
{
	TVector<T> const NewY = YAxis.GetSafeNormal();
	TVector<T> Norm = XAxis.GetSafeNormal();

	// if they're almost same, we need to find arbitrary vector
	if (FMath::IsNearlyEqual(FMath::Abs(NewY | Norm), T(1.f)))
	{
		// make sure we don't ever pick the same as NewX
		Norm = (FMath::Abs(NewY.Z) < (1.f - UE_KINDA_SMALL_NUMBER)) ? TVector<T>(0, 0, 1.f) : TVector<T>(1.f, 0, 0);
	}

	const TVector<T> NewZ = (Norm ^ NewY).GetSafeNormal();
	const TVector<T> NewX = NewY ^ NewZ;

	return TMatrix<T>(NewX, NewY, NewZ, TVector<T>::ZeroVector);
}

template<typename T>
TMatrix<T> TRotationMatrix<T>::MakeFromYZ(TVector<T> const& YAxis, TVector<T> const& ZAxis)
{
	TVector<T> const NewY = YAxis.GetSafeNormal();
	TVector<T> Norm = ZAxis.GetSafeNormal();

	// if they're almost same, we need to find arbitrary vector
	if (FMath::IsNearlyEqual(FMath::Abs(NewY | Norm), T(1.f)))
	{
		// make sure we don't ever pick the same as NewX
		Norm = (FMath::Abs(NewY.Z) < (1.f - UE_KINDA_SMALL_NUMBER)) ? TVector<T>(0, 0, 1.f) : TVector<T>(1.f, 0, 0);
	}

	const TVector<T> NewX = (NewY ^ Norm).GetSafeNormal();
	const TVector<T> NewZ = NewX ^ NewY;

	return TMatrix<T>(NewX, NewY, NewZ, TVector<T>::ZeroVector);
}

template<typename T>
TMatrix<T> TRotationMatrix<T>::MakeFromZX(TVector<T> const& ZAxis, TVector<T> const& XAxis)
{
	TVector<T> const NewZ = ZAxis.GetSafeNormal();
	TVector<T> Norm = XAxis.GetSafeNormal();

	// if they're almost same, we need to find arbitrary vector
	if (FMath::IsNearlyEqual(FMath::Abs(NewZ | Norm), T(1.f)))
	{
		// make sure we don't ever pick the same as NewX
		Norm = (FMath::Abs(NewZ.Z) < (1.f - UE_KINDA_SMALL_NUMBER)) ? TVector<T>(0, 0, 1.f) : TVector<T>(1.f, 0, 0);
	}

	const TVector<T> NewY = (NewZ ^ Norm).GetSafeNormal();
	const TVector<T> NewX = NewY ^ NewZ;

	return TMatrix<T>(NewX, NewY, NewZ, TVector<T>::ZeroVector);
}

template<typename T>
TMatrix<T> TRotationMatrix<T>::MakeFromZY(TVector<T> const& ZAxis, TVector<T> const& YAxis)
{
	TVector<T> const NewZ = ZAxis.GetSafeNormal();
	TVector<T> Norm = YAxis.GetSafeNormal();

	// if they're almost same, we need to find arbitrary vector
	if (FMath::IsNearlyEqual(FMath::Abs(NewZ | Norm), T(1.f)))
	{
		// make sure we don't ever pick the same as NewX
		Norm = (FMath::Abs(NewZ.Z) < (1.f - UE_KINDA_SMALL_NUMBER)) ? TVector<T>(0, 0, 1.f) : TVector<T>(1.f, 0, 0);
	}

	const TVector<T> NewX = (Norm ^ NewZ).GetSafeNormal();
	const TVector<T> NewY = NewZ ^ NewX;

	return TMatrix<T>(NewX, NewY, NewZ, TVector<T>::ZeroVector);
}

} // namespace Math
} // namespace UE

UE_DECLARE_LWC_TYPE(RotationMatrix, 44);

template<> struct TIsUECoreVariant<FRotationMatrix44f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FRotationMatrix44d> { enum { Value = true }; };