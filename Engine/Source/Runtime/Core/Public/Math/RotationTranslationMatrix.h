// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Math/VectorRegister.h"
#include "Math/Matrix.h"

namespace UE {
namespace Math {

/** Combined rotation and translation matrix */
template<typename T>
struct TRotationTranslationMatrix
	: public TMatrix<T>
{
public:
	using TMatrix<T>::M;

	/**
	 * Constructor.
	 *
	 * @param Rot rotation
	 * @param Origin translation to apply
	 */
	[[nodiscard]] TRotationTranslationMatrix(const TRotator<T>& Rot, const TVector<T>& Origin);

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	[[nodiscard]] explicit TRotationTranslationMatrix(const TRotationTranslationMatrix<FArg>& From)
		: TMatrix<T>(From)
	{
	}
	
	/** Matrix factory. Return an TMatrix<T> so we don't have type conversion issues in expressions. */
	[[nodiscard]] static TMatrix<T> Make(const TRotator<T>& Rot, const TVector<T>& Origin)
	{
		return TRotationTranslationMatrix(Rot, Origin);
	}
};


template<typename T>
inline TRotationTranslationMatrix<T>::TRotationTranslationMatrix(const TRotator<T>& Rot, const TVector<T>& Origin)
{
#if PLATFORM_ENABLE_VECTORINTRINSICS && (!PLATFORM_CPU_ARM_FAMILY)

	const TVectorRegisterType<T> Angles = MakeVectorRegister(Rot.Pitch, Rot.Yaw, Rot.Roll, 0.0f);
	const TVectorRegisterType<T> HalfAngles = VectorMultiply(Angles, GlobalVectorConstants::DEG_TO_RAD);

	union { TVectorRegisterType<T> v; T f[4]; } SinAngles, CosAngles;
	VectorSinCos(&SinAngles.v, &CosAngles.v, &HalfAngles);

	const T SP	= SinAngles.f[0];
	const T SY	= SinAngles.f[1];
	const T SR	= SinAngles.f[2];
	const T CP	= CosAngles.f[0];
	const T CY	= CosAngles.f[1];
	const T CR	= CosAngles.f[2];

#else

	T SP, SY, SR;
	T CP, CY, CR;
	FMath::SinCos(&SP, &CP, (T)FMath::DegreesToRadians(Rot.Pitch));
	FMath::SinCos(&SY, &CY, (T)FMath::DegreesToRadians(Rot.Yaw));
	FMath::SinCos(&SR, &CR, (T)FMath::DegreesToRadians(Rot.Roll));

#endif // PLATFORM_ENABLE_VECTORINTRINSICS

	M[0][0]	= CP * CY;
	M[0][1]	= CP * SY;
	M[0][2]	= SP;
	M[0][3]	= 0.f;

	M[1][0]	= SR * SP * CY - CR * SY;
	M[1][1]	= SR * SP * SY + CR * CY;
	M[1][2]	= - SR * CP;
	M[1][3]	= 0.f;

	M[2][0]	= -( CR * SP * CY + SR * SY );
	M[2][1]	= CY * SR - CR * SP * SY;
	M[2][2]	= CR * CP;
	M[2][3]	= 0.f;

	M[3][0]	= Origin.X;
	M[3][1]	= Origin.Y;
	M[3][2]	= Origin.Z;
	M[3][3]	= 1.f;
}

} // namespace Math
} // namespace UE

UE_DECLARE_LWC_TYPE(RotationTranslationMatrix, 44);

template<> struct TIsUECoreVariant<FRotationTranslationMatrix44f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FRotationTranslationMatrix44d> { enum { Value = true }; };