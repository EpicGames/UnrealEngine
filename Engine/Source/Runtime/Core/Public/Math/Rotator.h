// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/MathFwd.h" // IWYU pragma: export
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Misc/Parse.h"
#include "Misc/LargeWorldCoordinatesSerializer.h"
#include "Logging/LogMacros.h"
#include "Math/Vector.h"
#include "Math/VectorRegister.h"
#include "UObject/ObjectVersion.h"

namespace UE
{
namespace Math
{

/**
 * Implements a container for rotation information.
 *
 * All rotation values are stored in degrees.
 *
 * The angles are interpreted as intrinsic rotations applied in the order Yaw, then Pitch, then Roll. I.e., an object would be rotated
 * first by the specified yaw around its up axis (with positive angles interpreted as clockwise when viewed from above, along -Z), 
 * then pitched around its (new) right axis (with positive angles interpreted as 'nose up', i.e. clockwise when viewed along +Y), 
 * and then finally rolled around its (final) forward axis (with positive angles interpreted as clockwise rotations when viewed along +X).
 *
 * Note that these conventions differ from quaternion axis/angle. UE Quat always considers a positive angle to be a left-handed rotation, 
 * whereas Rotator treats yaw as left-handed but pitch and roll as right-handed.
 * 
 */
template<typename T>
struct TRotator
{

	// Can't have a UE_REQUIRES in the declaration because of the forward declarations, so check for allowed types here.
	static_assert(std::is_floating_point_v<T>, "TRotator only supports float and double types.");

public:
	using FReal = T;

	/** Rotation around the right axis (around Y axis), Looking up and down (0=Straight Ahead, +Up, -Down) */
	T Pitch;

	/** Rotation around the up axis (around Z axis), Turning around (0=Forward, +Right, -Left)*/
	T Yaw;

	/** Rotation around the forward axis (around X axis), Tilting your head, (0=Straight, +Clockwise, -CCW) */
	T Roll;

public:

	/** A rotator of zero degrees on each axis. */
	CORE_API static const TRotator<T> ZeroRotator;

public:

#if ENABLE_NAN_DIAGNOSTIC
	inline void DiagnosticCheckNaN() const
	{
		if (ContainsNaN())
		{
			logOrEnsureNanError(TEXT("TRotator contains NaN: %s"), *ToString());
		}
	}

	inline void DiagnosticCheckNaN(const TCHAR* Message) const
	{
		if (ContainsNaN())
		{
			logOrEnsureNanError(TEXT("%s: TRotator contains NaN: %s"), Message, *ToString());
		}
	}
#else
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN() const {}
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN(const TCHAR* Message) const {}
#endif

	/**
	 * Default constructor (no initialization).
	 */
	[[nodiscard]] TRotator() = default;

	/**
	 * Constructor
	 *
	 * @param InF Value to set all components to.
	 */
	[[nodiscard]] explicit UE_FORCEINLINE_HINT TRotator(T InF);

	/**
	 * Constructor.
	 *
	 * @param InPitch Pitch in degrees.
	 * @param InYaw Yaw in degrees.
	 * @param InRoll Roll in degrees.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TRotator( T InPitch, T InYaw, T InRoll );

	/**
	 * Constructor.
	 *
	 * @param EForceInit Force Init Enum.
	 */
	[[nodiscard]] explicit UE_FORCEINLINE_HINT TRotator( EForceInit );

	/**
	 * Constructor.
	 *
	 * @param Quat Quaternion used to specify rotation.
	 */
[[nodiscard]] explicit CORE_API TRotator( const TQuat<T>& Quat );

public:

	// Binary arithmetic operators.

	/**
	 * Get the result of adding a rotator to this.
	 *
	 * @param R The other rotator.
	 * @return The result of adding a rotator to this.
	 */
	[[nodiscard]] TRotator operator+( const TRotator<T>& R ) const;

	/**
	 * Get the result of subtracting a rotator from this.
	 *
	 * @param R The other rotator.
	 * @return The result of subtracting a rotator from this.
	 */
	[[nodiscard]] TRotator operator-( const TRotator<T>& R ) const;

	/**
	 * Get the result of scaling this rotator.
	 *
	 * @param Scale The scaling factor.
	 * @return The result of scaling.
	 */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	[[nodiscard]] UE_FORCEINLINE_HINT TRotator operator*( FArg Scale ) const
	{
		return TRotator(Pitch * Scale, Yaw * Scale, Roll * Scale);
	}

	/**
	 * Multiply this rotator by a scaling factor.
	 *
	 * @param Scale The scaling factor.
	 * @return Copy of the rotator after scaling.
	 */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	inline TRotator operator*=( FArg Scale )
	{
		Pitch = Pitch * Scale; Yaw = Yaw * Scale; Roll = Roll * Scale;
		DiagnosticCheckNaN();
		return *this;
	}

	// Binary comparison operators.

	/**
	 * Checks whether two rotators are identical. This checks each component for exact equality.
	 *
	 * @param R The other rotator.
	 * @return true if two rotators are identical, otherwise false.
	 * @see Equals()
	 */
	[[nodiscard]] bool operator==( const TRotator<T>& R ) const;

	/**
	 * Checks whether two rotators are different.
	 *
	 * @param V The other rotator.
	 * @return true if two rotators are different, otherwise false.
	 */
	[[nodiscard]] bool operator!=( const TRotator<T>& V ) const;

	// Assignment operators.

	/**
	 * Adds another rotator to this.
	 *
	 * @param R The other rotator.
	 * @return Copy of rotator after addition.
	 */
	TRotator operator+=( const TRotator<T>& R );

	/**
	 * Subtracts another rotator from this.
	 *
	 * @param R The other rotator.
	 * @return Copy of rotator after subtraction.
	 */
	TRotator operator-=( const TRotator<T>& R );

public:

	// Functions.

	/**
	 * Checks whether rotator is nearly zero within specified tolerance, when treated as an orientation.
	 * This means that TRotator(0, 0, 360) is "zero", because it is the same final orientation as the zero rotator.
	 *
	 * @param Tolerance Error Tolerance.
	 * @return true if rotator is nearly zero, within specified tolerance, otherwise false.
	 */
	[[nodiscard]] bool IsNearlyZero( T Tolerance = UE_KINDA_SMALL_NUMBER ) const;

	/**
	 * Checks whether this has exactly zero rotation, when treated as an orientation.
	 * This means that TRotator(0, 0, 360) is "zero", because it is the same final orientation as the zero rotator.
	 *
	 * @return true if this has exactly zero rotation, otherwise false.
	 */
	[[nodiscard]] bool IsZero() const;

	/**
	 * Checks whether two rotators are equal within specified tolerance, when treated as an orientation.
	 * This means that TRotator(0, 0, 360).Equals(TRotator(0,0,0)) is true, because they represent the same final orientation.
	 * It can compare only wound rotators (i.e. multiples of 360 degrees) that end up in a same rotation
	 * Rotators that represent the same final rotation, but get there via different intermediate rotations aren't equal
	 * i.e. TRotator(0, 45, 0).Equals(TRotator(180, 135, 180)) is false
	 *
	 * @param R The other rotator.
	 * @param Tolerance Error Tolerance.
	 * @return true if two rotators are equal, within specified tolerance, otherwise false.
	 */
	[[nodiscard]] bool Equals( const TRotator<T>& R, T Tolerance = UE_KINDA_SMALL_NUMBER ) const;

	/**
	 * Checks whether two rotators have the same orientation within the specified tolerance, without requiring similar angles or wound multiples of 360.
	 * Unlike Equals(), it can compare Rotators that represent the same final orientation, but get there via different intermediate rotations.
	 * i.e. If we compare two rotators with different angles, but same orientation like so:
	 * A = TRotator(0, 45, 0)
	 * B = TRotator(180, 135, 180)
	 * Then A.EqualsOrientation(B) would be TRUE while A.Equals(B) would be FALSE
	 *
	 * @param R The other rotator.
	 * @param Tolerance Error Tolerance.
	 * @return true if two rotators are equal, within specified tolerance, otherwise false.
	 */
	[[nodiscard]] bool EqualsOrientation( const TRotator<T>& R, T Tolerance = UE_KINDA_SMALL_NUMBER ) const;

	/**
	 * Adds to each component of the rotator.
	 *
	 * @param DeltaPitch Change in pitch. (+/-)
	 * @param DeltaYaw Change in yaw. (+/-)
	 * @param DeltaRoll Change in roll. (+/-)
	 * @return Copy of rotator after addition.
	 */
	TRotator Add( T DeltaPitch, T DeltaYaw, T DeltaRoll );

	/**
	 * Returns the inverse of the rotator.
	 */
	[[nodiscard]] CORE_API TRotator GetInverse() const;

	/**
	 * Get the rotation, snapped to specified degree segments.
	 *
	 * @param RotGrid A Rotator specifying how to snap each component.
	 * @return Snapped version of rotation.
	 */
	[[nodiscard]] TRotator GridSnap( const TRotator<T>& RotGrid ) const;

	/**
	 * Convert a rotation into a unit vector facing in its direction.
	 *
	 * @return Rotation as a unit direction vector.
	 */
	[[nodiscard]] CORE_API TVector<T> Vector() const;

	/**
	 * Get Rotation as a quaternion.
	 *
	 * @return Rotation as a quaternion.
	 */
	[[nodiscard]] CORE_API TQuat<T> Quaternion() const;

	/**
	 * Convert a Rotator into floating-point Euler angles (in degrees). Rotator now stored in degrees.
	 *
	 * @return Rotation as a Euler angle vector.
	 */
	[[nodiscard]] CORE_API TVector<T> Euler() const;

	/**
	 * Rotate a vector rotated by this rotator.
	 *
	 * @param V The vector to rotate.
	 * @return The rotated vector.
	 */
	[[nodiscard]] CORE_API TVector<T> RotateVector( const UE::Math::TVector<T>& V ) const;

	/**
	 * Returns the vector rotated by the inverse of this rotator.
	 *
	 * @param V The vector to rotate.
	 * @return The rotated vector.
	 */
	[[nodiscard]] CORE_API TVector<T> UnrotateVector( const UE::Math::TVector<T>& V ) const;

	/**
	 * Gets the rotation values so they fall within the range [0,360]
	 *
	 * @return Clamped version of rotator.
	 */
	[[nodiscard]] TRotator<T> Clamp() const;

	/** 
	 * Create a copy of this rotator and normalize, removes all winding and creates the "shortest route" rotation. 
	 *
	 * @return Normalized copy of this rotator
	 */
	[[nodiscard]] TRotator<T> GetNormalized() const;

	/** 
	 * Create a copy of this rotator and denormalize, clamping each axis to 0 - 360. 
	 *
	 * @return Denormalized copy of this rotator
	 */
	[[nodiscard]] TRotator<T> GetDenormalized() const;

	/** Get a specific component of the vector, given a specific axis by enum */
	[[nodiscard]] T GetComponentForAxis(EAxis::Type Axis) const;

	/** Set a specified componet of the vector, given a specific axis by enum */
	void SetComponentForAxis(EAxis::Type Axis, T Component);

	/**
	 * In-place normalize, removes all winding and creates the "shortest route" rotation.
	 */
	void Normalize();

	/** 
	 * Decompose this Rotator into a Winding part (multiples of 360) and a Remainder part. 
	 * Remainder will always be in [-180, 180] range.
	 *
	 * @param Winding[Out] the Winding part of this Rotator
	 * @param Remainder[Out] the Remainder
	 */
	CORE_API void GetWindingAndRemainder( TRotator<T>& Winding, TRotator<T>& Remainder ) const;

	/**
	* Return the manhattan distance in degrees between this Rotator and the passed in one.
	* @param Rotator[In] the Rotator we are comparing with.
	* @return Distance(Manhattan) between the two rotators. 
	*/
	[[nodiscard]] T GetManhattanDistance(const TRotator<T> & Rotator) const;

	/**
	* Return a Rotator that has the same rotation but has different degree values for Yaw, Pitch, and Roll.
	* This rotator should be within -180,180 range,
	* @return A Rotator with the same rotation but different degrees.
	*/
	[[nodiscard]] TRotator GetEquivalentRotator() const;

	/**
	* Modify if necessary the passed in rotator to be the closest rotator to it based upon it's equivalent.
	* This Rotator should be within (-180, 180], usually just constructed from a Matrix or a Quaternion.
	*
	* @param MakeClosest[In/Out] the Rotator we want to make closest to us. Should be between 
	* (-180, 180]. This Rotator may change if we need to use different degree values to make it closer.
	*/
	void SetClosestToMe(TRotator& MakeClosest) const;

	/**
	 * Get a textual representation of the vector.
	 *
	 * @return Text describing the vector.
	 */
	[[nodiscard]] FString ToString() const;

	/** Get a short textural representation of this vector, for compact readable logging. */
	[[nodiscard]] FString ToCompactString() const;

	/**
	 * Initialize this Rotator based on an FString. The String is expected to contain P=, Y=, R=.
	 * The TRotator will be bogus when InitFromString returns false.
	 *
	 * @param InSourceString	FString containing the rotator values.
	 * @return true if the P,Y,R values were read successfully; false otherwise.
	 */
	bool InitFromString( const FString& InSourceString );

	/**
	 * Utility to check if there are any non-finite values (NaN or Inf) in this Rotator.
	 *
	 * @return true if there are any non-finite values in this Rotator, otherwise false.
	 */
	[[nodiscard]] bool ContainsNaN() const;

	/**
	 * Serializes the rotator compressed for e.g. network transmission.
	 * 
	 * @param	Ar	Archive to serialize to/ from
	 */
	CORE_API void SerializeCompressed( FArchive& Ar );

	/**
	 * Serializes the rotator compressed for e.g. network transmission (use shorts though).
	 * 
	 * @param	Ar	Archive to serialize to/ from
	 */
	CORE_API void SerializeCompressedShort( FArchive& Ar );

	/**
	 */
	CORE_API bool NetSerialize( FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess );

public:
	
	/**
	 * Clamps an angle to the range of [0, 360).
	 *
	 * @param Angle The angle to clamp.
	 * @return The clamped angle.
	 */
	[[nodiscard]] static T ClampAxis( T Angle );

	/**
	 * Clamps an angle to the range of (-180, 180].
	 *
	 * @param Angle The Angle to clamp.
	 * @return The clamped angle.
	 */
	[[nodiscard]] static T NormalizeAxis( T Angle );

	/**
	 * Compresses a floating point angle into a byte.
	 *
	 * @param Angle The angle to compress.
	 * @return The angle as a byte.
	 */
	[[nodiscard]] static uint8 CompressAxisToByte( T Angle );

	/**
	 * Decompress a word into a floating point angle.
	 *
	 * @param Angle The word angle.
	 * @return The decompressed angle.
	 */
	[[nodiscard]] static T DecompressAxisFromByte( uint8 Angle );

	/**
	 * Compress a floating point angle into a word.
	 *
	 * @param Angle The angle to compress.
	 * @return The decompressed angle.
	 */
	[[nodiscard]] static uint16 CompressAxisToShort( T Angle );

	/**
	 * Decompress a short into a floating point angle.
	 *
	 * @param Angle The word angle.
	 * @return The decompressed angle.
	 */
	[[nodiscard]] static T DecompressAxisFromShort( uint16 Angle );

	/**
	 * Convert a vector of floating-point Euler angles (in degrees) into a Rotator. Rotator now stored in degrees
	 *
	 * @param Euler Euler angle vector.
	 * @return A rotator from a Euler angle.
	 */
	[[nodiscard]] static CORE_API TRotator MakeFromEuler( const TVector<T>& Euler );


public:

	bool Serialize( FArchive& Ar )
	{
		Ar << *this;
		return true;
	}

	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		if constexpr (std::is_same_v<T, float>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Rotator, Rotator3f, Rotator3d);
		}
		else
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Rotator, Rotator3d, Rotator3f);
		}
	}

	// Conversion from other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TRotator(const TRotator<FArg>& From)
		: TRotator<T>((T)From.Pitch, (T)From.Yaw, (T)From.Roll)
	{
	}
};

#if !defined(_MSC_VER) || defined(__clang__)  // MSVC can't forward declare explicit specializations
template<> CORE_API const FRotator3f FRotator3f::ZeroRotator;
template<> CORE_API const FRotator3d FRotator3d::ZeroRotator;
#endif

/* TRotator inline functions
 *****************************************************************************/

 /**
  * Serializer.
  *
  * @param Ar Serialization Archive.
  * @param R Rotator being serialized.
  * @return Reference to Archive after serialization.
  */

inline FArchive& operator<<(FArchive& Ar, TRotator<float>& R)
{
	Ar << R.Pitch << R.Yaw << R.Roll;
	return Ar;
}

inline FArchive& operator<<(FArchive& Ar, TRotator<double>& R)
{
	if (Ar.UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
	{
		Ar << R.Pitch << R.Yaw << R.Roll;
	}
	else
	{
		checkf(Ar.IsLoading(), TEXT("float -> double conversion applied outside of load!"));
		// Stored as floats, so serialize float and copy.
		float Pitch, Yaw, Roll;
		Ar << Pitch << Yaw << Roll;
		R = TRotator<double>(Pitch, Yaw, Roll);
	}
	return Ar;
}

/* FRotator inline functions
 *****************************************************************************/

/**
 * Scale a rotator and return.
 *
 * @param Scale scale to apply to R.
 * @param R rotator to be scaled.
 * @return Scaled rotator.
 */
template<typename T, typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
UE_FORCEINLINE_HINT TRotator<T> operator*(FArg Scale, const TRotator<T>& R )
{
	return R.operator*( Scale );
}

template<typename T>
inline TRotator<T>::TRotator( T InF ) 
	:	Pitch(InF), Yaw(InF), Roll(InF) 
{
	DiagnosticCheckNaN();
}

template<typename T>
inline TRotator<T>::TRotator( T InPitch, T InYaw, T InRoll )
	:	Pitch(InPitch), Yaw(InYaw), Roll(InRoll) 
{
	DiagnosticCheckNaN();
}

template<typename T>
UE_FORCEINLINE_HINT TRotator<T>::TRotator(EForceInit)
	: Pitch(0), Yaw(0), Roll(0)
{}

template<typename T>
UE_FORCEINLINE_HINT TRotator<T> TRotator<T>::operator+( const TRotator<T>& R ) const
{
	return TRotator( Pitch+R.Pitch, Yaw+R.Yaw, Roll+R.Roll );
}

template<typename T>
UE_FORCEINLINE_HINT TRotator<T> TRotator<T>::operator-( const TRotator<T>& R ) const
{
	return TRotator( Pitch-R.Pitch, Yaw-R.Yaw, Roll-R.Roll );
}

template<typename T>
UE_FORCEINLINE_HINT bool TRotator<T>::operator==( const TRotator<T>& R ) const
{
	return Pitch==R.Pitch && Yaw==R.Yaw && Roll==R.Roll;
}

template<typename T>
UE_FORCEINLINE_HINT bool TRotator<T>::operator!=( const TRotator<T>& V ) const
{
	return Pitch!=V.Pitch || Yaw!=V.Yaw || Roll!=V.Roll;
}

template<typename T>
inline TRotator<T> TRotator<T>::operator+=( const TRotator<T>& R )
{
	Pitch += R.Pitch; Yaw += R.Yaw; Roll += R.Roll;
	DiagnosticCheckNaN();
	return *this;
}

template<typename T>
inline TRotator<T> TRotator<T>::operator-=( const TRotator<T>& R )
{
	Pitch -= R.Pitch; Yaw -= R.Yaw; Roll -= R.Roll;
	DiagnosticCheckNaN();
	return *this;
}

template<typename T>
inline bool TRotator<T>::IsNearlyZero(T Tolerance) const
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	const TVectorRegisterType<T> RegA = VectorLoadFloat3_W0(this);
	const TVectorRegisterType<T> Norm = VectorNormalizeRotator(RegA);
	const TVectorRegisterType<T> AbsNorm = VectorAbs(Norm);
	return !VectorAnyGreaterThan(AbsNorm, VectorLoadFloat1(&Tolerance));
#else
	return
		FMath::Abs(NormalizeAxis(Pitch))<=Tolerance
		&&	FMath::Abs(NormalizeAxis(Yaw))<=Tolerance
		&&	FMath::Abs(NormalizeAxis(Roll))<=Tolerance;
#endif
}

template<typename T>
UE_FORCEINLINE_HINT bool TRotator<T>::IsZero() const
{
	return (ClampAxis(Pitch)==0.f) && (ClampAxis(Yaw)==0.f) && (ClampAxis(Roll)==0.f);
}

template<typename T>
inline bool TRotator<T>::Equals(const TRotator<T>& R, T Tolerance) const
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	const TVectorRegisterType<T> RegA = VectorLoadFloat3_W0(this);
	const TVectorRegisterType<T> RegB = VectorLoadFloat3_W0(&R);
	const TVectorRegisterType<T> NormDelta = VectorNormalizeRotator(VectorSubtract(RegA, RegB));
	const TVectorRegisterType<T> AbsNormDelta = VectorAbs(NormDelta);
	return !VectorAnyGreaterThan(AbsNormDelta, VectorLoadFloat1(&Tolerance));
#else
	return (FMath::Abs(NormalizeAxis(Pitch - R.Pitch)) <= Tolerance) 
		&& (FMath::Abs(NormalizeAxis(Yaw - R.Yaw)) <= Tolerance) 
		&& (FMath::Abs(NormalizeAxis(Roll - R.Roll)) <= Tolerance);
#endif
}

template <typename T>
bool TRotator<T>::EqualsOrientation(const TRotator<T>& R, T Tolerance) const
{
	return Quaternion().AngularDistance(R.Quaternion()) <= Tolerance;
}

template<typename T>
inline TRotator<T> TRotator<T>::Add( T DeltaPitch, T DeltaYaw, T DeltaRoll )
{
	Yaw   += DeltaYaw;
	Pitch += DeltaPitch;
	Roll  += DeltaRoll;
	DiagnosticCheckNaN();
	return *this;
}

template<typename T>
inline TRotator<T> TRotator<T>::GridSnap( const TRotator<T>& RotGrid ) const
{
	return TRotator
		(
		FMath::GridSnap(Pitch,RotGrid.Pitch),
		FMath::GridSnap(Yaw,  RotGrid.Yaw),
		FMath::GridSnap(Roll, RotGrid.Roll)
		);
}

template<typename T>
UE_FORCEINLINE_HINT TRotator<T> TRotator<T>::Clamp() const
{
	return TRotator(ClampAxis(Pitch), ClampAxis(Yaw), ClampAxis(Roll));
}

template<typename T>
inline T TRotator<T>::ClampAxis( T Angle )
{
	// returns Angle in the range (-360,360)
	Angle = FMath::Fmod(Angle, (T)360.0);

	if (Angle < (T)0.0)
	{
		// shift to [0,360) range
		Angle += (T)360.0;
	}

	return Angle;
}

template<typename T>
inline T TRotator<T>::NormalizeAxis( T Angle )
{
	// returns Angle in the range [0,360)
	Angle = ClampAxis(Angle);

	if (Angle > (T)180.0)
	{
		// shift to (-180,180]
		Angle -= (T)360.0;
	}

	return Angle;
}

template<typename T>
UE_FORCEINLINE_HINT uint8 TRotator<T>::CompressAxisToByte( T Angle )
{
	// map [0->360) to [0->256) and mask off any winding
	return FMath::RoundToInt(Angle * (T)256.f / (T)360.f) & 0xFF;
}

template<typename T>
UE_FORCEINLINE_HINT T TRotator<T>::DecompressAxisFromByte( uint8 Angle )
{
	// map [0->256) to [0->360)
	return (Angle * (T)360.f / (T)256.f);
}

template<typename T>
UE_FORCEINLINE_HINT uint16 TRotator<T>::CompressAxisToShort( T Angle )
{
	// map [0->360) to [0->65536) and mask off any winding
	return FMath::RoundToInt(Angle * (T)65536.f / (T)360.f) & 0xFFFF;
}

template<typename T>
UE_FORCEINLINE_HINT T TRotator<T>::DecompressAxisFromShort( uint16 Angle )
{
	// map [0->65536) to [0->360)
	return (Angle * (T)360.f / (T)65536.f);
}

template<typename T>
inline TRotator<T> TRotator<T>::GetNormalized() const
{
	TRotator Rot = *this;
	Rot.Normalize();
	return Rot;
}

template<typename T>
inline TRotator<T> TRotator<T>::GetDenormalized() const
{
	TRotator Rot = *this;
	Rot.Pitch	= ClampAxis(Rot.Pitch);
	Rot.Yaw		= ClampAxis(Rot.Yaw);
	Rot.Roll	= ClampAxis(Rot.Roll);
	return Rot;
}

template<typename T>
inline void TRotator<T>::Normalize()
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	TVectorRegisterType<T> VRotator = VectorLoadFloat3_W0(this);
	VRotator = VectorNormalizeRotator(VRotator);
	VectorStoreFloat3(VRotator, this);
#else
	Pitch = NormalizeAxis(Pitch);
	Yaw = NormalizeAxis(Yaw);
	Roll = NormalizeAxis(Roll);
#endif
	DiagnosticCheckNaN();
}

template<typename T>
inline T TRotator<T>::GetComponentForAxis(EAxis::Type Axis) const
{
	switch (Axis)
	{
	case EAxis::X:
		return Roll;
	case EAxis::Y:
		return Pitch;
	case EAxis::Z:
		return Yaw;
	case EAxis::None:
	default:
		return 0.f;
	}
}

template<typename T>
inline void TRotator<T>::SetComponentForAxis(EAxis::Type Axis, T Component)
{
	switch (Axis)
	{
	case EAxis::X:
		Roll = Component;
		break;
	case EAxis::Y:
		Pitch = Component;
		break;
	case EAxis::Z:
		Yaw = Component;
		break;
	case EAxis::None:
	default:
		break;
	}
}

template<typename T>
UE_FORCEINLINE_HINT FString TRotator<T>::ToString() const
{
	return FString::Printf(TEXT("P=%f Y=%f R=%f"), Pitch, Yaw, Roll );
}

template<typename T>
inline FString TRotator<T>::ToCompactString() const
{
	if( IsNearlyZero() )
	{
		return FString("R(0)");
	}

	FString ReturnString("R(");
	bool bIsEmptyString = true;
	if( !FMath::IsNearlyZero(Pitch) )
	{
		ReturnString += FString::Printf(TEXT("P=%.2f"), Pitch);
		bIsEmptyString = false;
	}
	if( !FMath::IsNearlyZero(Yaw) )
	{
		if( !bIsEmptyString )
		{
			ReturnString += FString(", ");
		}
		ReturnString += FString::Printf(TEXT("Y=%.2f"), Yaw);
		bIsEmptyString = false;
	}
	if( !FMath::IsNearlyZero(Roll) )
	{
		if( !bIsEmptyString )
		{
			ReturnString += FString(", ");
		}
		ReturnString += FString::Printf(TEXT("R=%.2f"), Roll);
		bIsEmptyString = false;
	}
	ReturnString += FString(")");
	return ReturnString;
}

template<typename T>
inline bool TRotator<T>::InitFromString( const FString& InSourceString )
{
	Pitch = Yaw = Roll = 0;

	// The initialization is only successful if the X, Y, and Z values can all be parsed from the string
	const bool bSuccessful = FParse::Value( *InSourceString, TEXT("P=") , Pitch ) && FParse::Value( *InSourceString, TEXT("Y="), Yaw ) && FParse::Value( *InSourceString, TEXT("R="), Roll );
	DiagnosticCheckNaN();
	return bSuccessful;
}

template<typename T>
inline bool TRotator<T>::ContainsNaN() const
{
	return (!FMath::IsFinite(Pitch) ||
			!FMath::IsFinite(Yaw) ||
			!FMath::IsFinite(Roll));
}

template<typename T>
UE_FORCEINLINE_HINT T TRotator<T>::GetManhattanDistance(const TRotator<T> & Rotator) const
{
	return FMath::Abs<T>(Yaw - Rotator.Yaw) + FMath::Abs<T>(Pitch - Rotator.Pitch) + FMath::Abs<T>(Roll - Rotator.Roll);
}

template<typename T>
UE_FORCEINLINE_HINT TRotator<T> TRotator<T>::GetEquivalentRotator() const
{
	return TRotator(180.0f - Pitch,Yaw + 180.0f, Roll + 180.0f);
}

template<typename T>
inline void TRotator<T>::SetClosestToMe(TRotator& MakeClosest) const
{
	TRotator OtherChoice = MakeClosest.GetEquivalentRotator();
	T FirstDiff = GetManhattanDistance(MakeClosest);
	T SecondDiff = GetManhattanDistance(OtherChoice);
	if (SecondDiff < FirstDiff)
		MakeClosest = OtherChoice;
}

} // namespace UE::Math
} // namespace UE

UE_DECLARE_LWC_TYPE(Rotator, 3);

template<> struct TCanBulkSerialize<FRotator3f> { enum { Value = false }; };
template<> struct TIsPODType<FRotator3f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FRotator3f> { enum { Value = true }; };
DECLARE_INTRINSIC_TYPE_LAYOUT(FRotator3f);

template<> struct TCanBulkSerialize<FRotator3d> { enum { Value = false }; };
template<> struct TIsPODType<FRotator3d> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FRotator3d> { enum { Value = true }; };
DECLARE_INTRINSIC_TYPE_LAYOUT(FRotator3d);

// Forward declare all explicit specializations (in UnrealMath.cpp)
template<> CORE_API FQuat4f FRotator3f::Quaternion() const;
template<> CORE_API FQuat4d FRotator3d::Quaternion() const;


/* FMath inline functions
 *****************************************************************************/

template<typename T>
struct TCustomLerp<UE::Math::TRotator<T>>
{
	// Required to make FMath::Lerp<TRotator>() call our custom Lerp() implementation below.
	constexpr static bool Value = true;
	using RotatorType = UE::Math::TRotator<T>;

	template<class U>
	static UE_FORCEINLINE_HINT RotatorType Lerp(const RotatorType& A, const RotatorType& B, const U& Alpha)
	{
		return A + (B - A).GetNormalized() * Alpha;
	}
};

template< typename T, typename U >
UE_FORCEINLINE_HINT UE::Math::TRotator<T> FMath::LerpRange(const UE::Math::TRotator<T>& A, const UE::Math::TRotator<T>& B, U Alpha)
{
	// Similar to Lerp, but does not take the shortest path. Allows interpolation over more than 180 degrees.
	return (A * ((T)1.0 - (T)Alpha) + B * Alpha).GetNormalized();
}

template<typename T>
inline T FMath::ClampAngle(T AngleDegrees, T MinAngleDegrees, T MaxAngleDegrees)
{
	const T MaxDelta = UE::Math::TRotator<T>::ClampAxis(MaxAngleDegrees - MinAngleDegrees) * 0.5f;			// 0..180
	const T RangeCenter = UE::Math::TRotator<T>::ClampAxis(MinAngleDegrees + MaxDelta);						// 0..360
	const T DeltaFromCenter = UE::Math::TRotator<T>::NormalizeAxis(AngleDegrees - RangeCenter);				// -180..180

	// maybe clamp to nearest edge
	if (DeltaFromCenter > MaxDelta)
	{
		return UE::Math::TRotator<T>::NormalizeAxis(RangeCenter + MaxDelta);
	}
	else if (DeltaFromCenter < -MaxDelta)
	{
		return UE::Math::TRotator<T>::NormalizeAxis(RangeCenter - MaxDelta);
	}

	// already in range, just return it
	return UE::Math::TRotator<T>::NormalizeAxis(AngleDegrees);
}
