// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Math/Color.h"
#include "Math/MathFwd.h"
#include "Math/UnrealMathSSE.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Math/VectorRegister.h"

class FArchive;

//	Constants.
extern CORE_API float NormalizationConstants[9];
extern CORE_API int32 BasisL[9];
extern CORE_API int32 BasisM[9];

extern CORE_API float LegendrePolynomial(int32 L, int32 M, float X);

extern CORE_API FLinearColor SHGetLuminanceFactors();

/** Returns the basis index of the SH basis L,M. */
UE_FORCEINLINE_HINT int32 SHGetBasisIndex(int32 L,int32 M)
{
	return L * (L + 1) + M;
}

/** A vector of spherical harmonic coefficients. */
template<int32 Order> 
class MS_ALIGN(16) TSHVector
{
public:

	enum { MaxSHOrder = Order };
	enum { MaxSHBasis = MaxSHOrder * MaxSHOrder };
	enum { NumComponentsPerSIMDVector = 4 };
	enum { NumSIMDVectors = (MaxSHBasis + NumComponentsPerSIMDVector - 1) / NumComponentsPerSIMDVector };
	enum { NumTotalFloats = NumSIMDVectors * NumComponentsPerSIMDVector };
	float V[NumTotalFloats];

	/** The integral of the constant SH basis. */
	static constexpr float ConstantBasisIntegral = 3.5449077018110320545963349666823f; // 2 * Sqrt(PI)

	/** Default constructor. */
	[[nodiscard]] TSHVector()
	{
		FMemory::Memzero(V,sizeof(V));
	}

	[[nodiscard]] TSHVector(float V0, float V1, float V2, float V3)
	{
		FMemory::Memzero(V,sizeof(V));

		V[0] = V0;
		V[1] = V1;
		V[2] = V2;
		V[3] = V3;
	}

	[[nodiscard]] explicit TSHVector(const FVector4f& Vector)
	{
		FMemory::Memzero(V,sizeof(V));

		V[0] = Vector.X;
		V[1] = Vector.Y;
		V[2] = Vector.Z;
		V[3] = Vector.W;
	}


	template<int32 OtherOrder>
	[[nodiscard]] explicit TSHVector(const TSHVector<OtherOrder>& Other)
	{
		if (Order <= OtherOrder)
		{
			FMemory::Memcpy(V, Other.V, sizeof(V));
		}
		else
		{
			FMemory::Memzero(V);
			FMemory::Memcpy(V, Other.V, sizeof(V));
		}
	}	

	/** Scalar multiplication operator. */
	/** Changed to float& from float to avoid LHS **/
	[[nodiscard]] friend inline TSHVector operator*(const TSHVector& A,const float& B)
	{
		const VectorRegister4Float ReplicatedScalar = VectorLoadFloat1(&B);

		TSHVector Result;
		for(int32 BasisIndex = 0;BasisIndex < NumSIMDVectors;BasisIndex++)
		{
			VectorRegister4Float MulResult = VectorMultiply(
				VectorLoadAligned(&A.V[BasisIndex * NumComponentsPerSIMDVector]),
				ReplicatedScalar
				);
			VectorStoreAligned(MulResult, &Result.V[BasisIndex * NumComponentsPerSIMDVector]);
		}
		return Result;
	}

	/** Scalar division operator. */
	[[nodiscard]] friend inline TSHVector operator/(const TSHVector& A,const float& Scalar)
	{
		const float B = (1.0f / Scalar);
		const VectorRegister4Float ReplicatedScalar = VectorLoadFloat1(&B);

		TSHVector Result;
		for(int32 BasisIndex = 0;BasisIndex < NumSIMDVectors;BasisIndex++)
		{
			VectorRegister4Float MulResult = VectorMultiply(
				VectorLoadAligned(&A.V[BasisIndex * NumComponentsPerSIMDVector]),
				ReplicatedScalar
				);
			VectorStoreAligned(MulResult, &Result.V[BasisIndex * NumComponentsPerSIMDVector]);
		}
		return Result;
	}

	/** Addition operator. */
	[[nodiscard]] friend inline TSHVector operator+(const TSHVector& A,const TSHVector& B)
	{
		TSHVector Result;
		for(int32 BasisIndex = 0;BasisIndex < NumSIMDVectors;BasisIndex++)
		{
			VectorRegister4Float AddResult = VectorAdd(
				VectorLoadAligned(&A.V[BasisIndex * NumComponentsPerSIMDVector]),
				VectorLoadAligned(&B.V[BasisIndex * NumComponentsPerSIMDVector])
				);

			VectorStoreAligned(AddResult, &Result.V[BasisIndex * NumComponentsPerSIMDVector]);
		}
		return Result;
	}
	
	/** Subtraction operator. */
	[[nodiscard]] friend inline TSHVector operator-(const TSHVector& A,const TSHVector& B)
	{
		TSHVector Result;
		for(int32 BasisIndex = 0;BasisIndex < NumSIMDVectors;BasisIndex++)
		{
			VectorRegister4Float SubResult = VectorSubtract(
				VectorLoadAligned(&A.V[BasisIndex * NumComponentsPerSIMDVector]),
				VectorLoadAligned(&B.V[BasisIndex * NumComponentsPerSIMDVector])
				);

			VectorStoreAligned(SubResult, &Result.V[BasisIndex * NumComponentsPerSIMDVector]);
		}
		return Result;
	}

	/** Dot product operator. */
	[[nodiscard]] friend inline float Dot(const TSHVector& A,const TSHVector& B)
	{
		VectorRegister4Float ReplicatedResult = VectorZero();
		for(int32 BasisIndex = 0;BasisIndex < NumSIMDVectors;BasisIndex++)
		{
			ReplicatedResult = VectorAdd(
				ReplicatedResult,
				VectorDot4(
					VectorLoadAligned(&A.V[BasisIndex * NumComponentsPerSIMDVector]),
					VectorLoadAligned(&B.V[BasisIndex * NumComponentsPerSIMDVector])
					)
				);
		}
		float Result;
		VectorStoreFloat1(ReplicatedResult,&Result);
		return Result;
	}

	/** In-place addition operator. */
	/** Changed from (*this = *this + B;} to calculate here to avoid LHS **/
	/** Now this avoids TSHVector + operator thus LHS on *this as well as Result and more **/
	inline TSHVector& operator+=(const TSHVector& B)
	{
		for(int32 BasisIndex = 0;BasisIndex < NumSIMDVectors;BasisIndex++)
		{
			VectorRegister4Float AddResult = VectorAdd(
				VectorLoadAligned(&V[BasisIndex * NumComponentsPerSIMDVector]),
				VectorLoadAligned(&B.V[BasisIndex * NumComponentsPerSIMDVector])
				);

			VectorStoreAligned(AddResult, &V[BasisIndex * NumComponentsPerSIMDVector]);
		}
		return *this;
	}
	
	/** In-place subtraction operator. */
	/** Changed from (*this = *this - B;} to calculate here to avoid LHS **/
	/** Now this avoids TSHVector - operator thus LHS on *this as well as Result and **/
	inline TSHVector& operator-=(const TSHVector& B)
	{
		for(int32 BasisIndex = 0;BasisIndex < NumSIMDVectors;BasisIndex++)
		{
			VectorRegister4Float SubResult = VectorSubtract(
				VectorLoadAligned(&V[BasisIndex * NumComponentsPerSIMDVector]),
				VectorLoadAligned(&B.V[BasisIndex * NumComponentsPerSIMDVector])
				);

			VectorStoreAligned(SubResult, &V[BasisIndex * NumComponentsPerSIMDVector]);
		}
		return *this;
	}

	/** In-place scalar division operator. */
	/** Changed to float& from float to avoid LHS **/
	/** Changed from (*this = *this * (1.0f/B);) to calculate here to avoid LHS **/
	/** Now this avoids TSHVector * operator thus LHS on *this as well as Result and LHS **/
	inline TSHVector& operator/=(const float& Scalar)
	{
		const float B = (1.0f/Scalar);
		const VectorRegister4Float ReplicatedScalar = VectorLoadFloat1(&B);

		for(int32 BasisIndex = 0;BasisIndex < NumSIMDVectors;BasisIndex++)
		{
			VectorRegister4Float MulResult = VectorMultiply(
				VectorLoadAligned(&V[BasisIndex * NumComponentsPerSIMDVector]),
				ReplicatedScalar
				);
			VectorStoreAligned(MulResult, &V[BasisIndex * NumComponentsPerSIMDVector]);
		}
		return *this;
	}

	/** In-place scalar multiplication operator. */
	/** Changed to float& from float to avoid LHS **/
	/** Changed from (*this = *this * B;) to calculate here to avoid LHS **/
	/** Now this avoids TSHVector * operator thus LHS on *this as well as Result and LHS **/
	inline TSHVector& operator*=(const float& B)
	{
		const VectorRegister4Float ReplicatedScalar = VectorLoadFloat1(&B);

		for(int32 BasisIndex = 0;BasisIndex < NumSIMDVectors;BasisIndex++)
		{
			VectorRegister4Float MulResult = VectorMultiply(
				VectorLoadAligned(&V[BasisIndex * NumComponentsPerSIMDVector]),
				ReplicatedScalar
				);
			VectorStoreAligned(MulResult, &V[BasisIndex * NumComponentsPerSIMDVector]);
		}
		return *this;
	}

	friend FArchive& operator<<(FArchive& Ar, TSHVector& SH)
	{
		for (int32 BasisIndex = 0; BasisIndex < MaxSHBasis; BasisIndex++)
		{
			Ar << SH.V[BasisIndex];
		}

		return Ar;
	}

	/** Calculates the integral of the function over the surface of the sphere. */
	[[nodiscard]] float CalcIntegral() const
	{
		return V[0] * ConstantBasisIntegral;
	}

	/** Scales the function uniformly so its integral equals one. */
	void Normalize()
	{
		const float Integral = CalcIntegral();
		if(Integral > UE_DELTA)
		{
			*this /= Integral;
		}
	}

	void ApplyWindowing(float Lambda)
	{
		// "Stupid Spherical Harmonics (SH) Tricks"
		// Minimizing the weighted squared Laplacian

		for (int32 l = 0; l < TSHVector::MaxSHOrder; l++)
		{
			const float BandScaleFactor = 1.0f / (1.0f + Lambda * float(l * l * (l + 1) * (l + 1)));

			for (int32 m = -l; m <= l; m++)
			{
				V[SHGetBasisIndex(l, m)] *= BandScaleFactor;
			}
		}
	}

	[[nodiscard]] bool AreFloatsValid() const
	{
		bool bValid = true;

		for (int32 BasisIndex = 0; BasisIndex < MaxSHBasis; BasisIndex++)
		{
			bValid = bValid && FMath::IsFinite(V[BasisIndex]) && !FMath::IsNaN(V[BasisIndex]);
		}
		
		return bValid;
	}

	/** Compute the direction which the spherical harmonic is highest at. */
	[[nodiscard]] FVector GetMaximumDirection() const
	{
		// This is an approximation which only takes into account first and second order spherical harmonics.
		return FVector(-V[3], -V[1], V[2]).GetSafeNormal();
	}

	[[nodiscard]] static TSHVector CalcDiffuseTransfer(const FVector& Normal)
	{
		TSHVector Result = SHBasisFunction(Normal);

		// These formula are scaling factors for each SH band that convolve a SH with the circularly symmetric function
		// max(0,cos(theta))
		float L0 =					UE_PI;
		float L1 =					2 * UE_PI / 3;
		float L2 =					UE_PI / 4;

		// Multiply the coefficients in each band with the appropriate band scaling factor.
		for(int32 BasisIndex = 0;BasisIndex < MaxSHBasis;BasisIndex++)
		{
			float Scale = 1.0f;

			if (BasisIndex < 1)
			{
				Scale = L0;
			}
			else if (BasisIndex < 4)
			{
				Scale = L1;
			}
			else
			{
				Scale = L2;
			}

			Result.V[BasisIndex] *= Scale;
		}

		return Result;
	}

	/** Returns the value of the SH basis L,M at the point on the sphere defined by the unit vector Vector. */
	[[nodiscard]] static TSHVector SHBasisFunction(const FVector& Vector)
	{
		TSHVector Result;

		// Initialize the result to the normalization constant.
		for (int32 BasisIndex = 0; BasisIndex < TSHVector::MaxSHBasis; BasisIndex++)
		{
			Result.V[BasisIndex] = NormalizationConstants[BasisIndex];
		}

		// Multiply the result by the phi-dependent part of the SH bases.
		// Skip this for X=0 and Y=0, because atan will be undefined and
		// we know the Vector will be (0,0,+1) or (0,0,-1).
		if (FMath::Abs(Vector.X) > UE_KINDA_SMALL_NUMBER || FMath::Abs(Vector.Y) > UE_KINDA_SMALL_NUMBER)
		{
			const float	Phi = (float)FMath::Atan2(Vector.Y, Vector.X);

			for (int32 BandIndex = 1; BandIndex < TSHVector::MaxSHOrder; BandIndex++)
			{
				const float	SinPhiM = FMath::Sin((float)BandIndex * Phi);
				const float	CosPhiM = FMath::Cos((float)BandIndex * Phi);

				for (int32 RecurrentBandIndex = BandIndex; RecurrentBandIndex < TSHVector::MaxSHOrder; RecurrentBandIndex++)
				{
					Result.V[SHGetBasisIndex(RecurrentBandIndex, -BandIndex)] *= SinPhiM;
					Result.V[SHGetBasisIndex(RecurrentBandIndex, +BandIndex)] *= CosPhiM;
				}
			}
		}

		// Multiply the result by the theta-dependent part of the SH bases.
		for (int32 BasisIndex = 1; BasisIndex < TSHVector::MaxSHBasis; BasisIndex++)
		{
			Result.V[BasisIndex] *= LegendrePolynomial(BasisL[BasisIndex], FMath::Abs(BasisM[BasisIndex]), (float)Vector.Z);
		}

		return Result;
	}

	/** The ambient incident lighting function. */
	[[nodiscard]] static TSHVector AmbientFunction()
	{
		TSHVector AmbientFunctionSH;
		AmbientFunctionSH.V[0] = 1.0f / (2.0f * FMath::Sqrt(UE_PI));
		return AmbientFunctionSH;
	}

	[[nodiscard]] static float FindWindowingLambda(const TSHVector& Vector, float TargetLaplacian)
	{
		// "Stupid Spherical Harmonics (SH) Tricks"
		// Appendix A7: Solving for Lamba to Reduce the Squared Laplacian

		float TableL[TSHVector::MaxSHOrder];
		float TableB[TSHVector::MaxSHOrder];

		TableL[0] = 0.0f;
		TableB[0] = 0.0f;

		for (int32 l = 1; l < TSHVector::MaxSHOrder; l++)
		{
			TableL[l] = float(l * l * (l + 1) * (l + 1));

			float B = 0.0f;
			for (int32 m = -1; m <= l; m++)
			{
				float Coefficient = Vector.V[SHGetBasisIndex(l, m)];
				B += Coefficient * Coefficient;
			}
			TableB[l] = B;
		}

		float SquaredLaplacian = 0.0f;

		for (int32 l = 1; l < TSHVector::MaxSHOrder; ++l)
		{
			SquaredLaplacian += TableL[l] * TableB[l];
		}

		const float TargetSquaredLaplacian = TargetLaplacian * TargetLaplacian;
		if (SquaredLaplacian <= TargetSquaredLaplacian)
		{
			return 0.0f;
		}

		float Lambda = 0.0f;

		const uint32 IterationLimit = 100;
		for (uint32 i = 0; i < IterationLimit; i++)
		{
			float f = 0.0f;
			float fd = 0.0f;

			for (int32 l = 1; l < TSHVector::MaxSHOrder; ++l)
			{
				float Temp = 1.0f + Lambda * TableL[l];
				f += TableL[l] * TableB[l] / (Temp * Temp);
				fd += (2.0f * TableL[l] * TableL[l] * TableB[l]) / (Temp * Temp * Temp);
			}

			f = TargetSquaredLaplacian - f;

			float delta = -f / fd;
			Lambda += delta;

			if (FMath::Abs(delta) < UE_KINDA_SMALL_NUMBER)
			{
				break;
			}
		}

		return Lambda;
	}
} GCC_ALIGN(16);


/** Specialization for 2nd order to avoid expensive trig functions. */
template<> 
[[nodiscard]] inline TSHVector<2> TSHVector<2>::SHBasisFunction(const FVector& Vector) 
{
	TSHVector<2> Result;
	Result.V[0] = 0.282095f; 
	Result.V[1] = -0.488603f * (float)Vector.Y;	// LWC_TODO: Precision loss
	Result.V[2] = 0.488603f * (float)Vector.Z;
	Result.V[3] = -0.488603f * (float)Vector.X;
	return Result;
}

/** Specialization for 3rd order to avoid expensive trig functions. */
template<> 
[[nodiscard]] inline TSHVector<3> TSHVector<3>::SHBasisFunction(const FVector& Vector) 
{
	TSHVector<3> Result;
	Result.V[0] = 0.282095f; 
	Result.V[1] = -0.488603f * (float)Vector.Y;	// LWC_TODO: Precision loss
	Result.V[2] = 0.488603f * (float)Vector.Z;
	Result.V[3] = -0.488603f * (float)Vector.X;

	FVector VectorSquared = Vector * Vector;
	Result.V[4] = 1.092548f * float(Vector.X * Vector.Y);
	Result.V[5] = -1.092548f * float(Vector.Y * Vector.Z);
	Result.V[6] = 0.315392f * float(3.0f * VectorSquared.Z - 1.0f);
	Result.V[7] = -1.092548f * float(Vector.X * Vector.Z);
	Result.V[8] = 0.546274f * float(VectorSquared.X - VectorSquared.Y);
	return Result;
}

/** A vector of colored spherical harmonic coefficients. */
template<int32 MaxSHOrder>
class TSHVectorRGB
{
public:

	TSHVector<MaxSHOrder> R;
	TSHVector<MaxSHOrder> G;
	TSHVector<MaxSHOrder> B;

	TSHVectorRGB() {}

	template<int32 OtherOrder>
	explicit TSHVectorRGB(const TSHVectorRGB<OtherOrder>& Other)
	{
		R = (TSHVector<MaxSHOrder>)Other.R;
		G = (TSHVector<MaxSHOrder>)Other.G;
		B = (TSHVector<MaxSHOrder>)Other.B;
	}

	/** Calculates greyscale spherical harmonic coefficients. */
	[[nodiscard]] TSHVector<MaxSHOrder> GetLuminance() const
	{
		static const FLinearColor LuminanceFactors = SHGetLuminanceFactors();

		return R * LuminanceFactors.R + G * LuminanceFactors.G + B * LuminanceFactors.B;
	}

	void Desaturate(float DesaturateFraction)
	{
		TSHVector<MaxSHOrder> Desaturated = GetLuminance() * DesaturateFraction;

		R = R * (1 - DesaturateFraction) + Desaturated;
		G = G * (1 - DesaturateFraction) + Desaturated;
		B = B * (1 - DesaturateFraction) + Desaturated;
	}

	/** Calculates the integral of the function over the surface of the sphere. */
	[[nodiscard]] FLinearColor CalcIntegral() const
	{
		FLinearColor Result;
		Result.R = R.CalcIntegral();
		Result.G = G.CalcIntegral();
		Result.B = B.CalcIntegral();
		Result.A = 1.0f;
		return Result;
	}

	void ApplyWindowing(float Lambda)
	{
		R.ApplyWindowing(Lambda);
		G.ApplyWindowing(Lambda);
		B.ApplyWindowing(Lambda);
	}

	[[nodiscard]] bool AreFloatsValid() const
	{
		return R.AreFloatsValid() && G.AreFloatsValid() && B.AreFloatsValid();
	}

	/** Scalar multiplication operator. */
	/** Changed to float& from float to avoid LHS **/
	[[nodiscard]] friend inline TSHVectorRGB operator*(const TSHVectorRGB& A, const float& Scalar)
	{
		TSHVectorRGB Result;
		Result.R = A.R * Scalar;
		Result.G = A.G * Scalar;
		Result.B = A.B * Scalar;
		return Result;
	}

	/** Scalar multiplication operator. */
	/** Changed to float& from float to avoid LHS **/
	[[nodiscard]] friend inline TSHVectorRGB operator*(const float& Scalar,const TSHVectorRGB& A)
	{
		TSHVectorRGB Result;
		Result.R = A.R * Scalar;
		Result.G = A.G * Scalar;
		Result.B = A.B * Scalar;
		return Result;
	}

	/** Color multiplication operator. */
	[[nodiscard]] friend inline TSHVectorRGB operator*(const TSHVectorRGB& A,const FLinearColor& Color)
	{
		TSHVectorRGB Result;
		Result.R = A.R * Color.R;
		Result.G = A.G * Color.G;
		Result.B = A.B * Color.B;
		return Result;
	}

	/** Color multiplication operator. */
	[[nodiscard]] friend inline TSHVectorRGB operator*(const FLinearColor& Color,const TSHVectorRGB& A)
	{
		TSHVectorRGB Result;
		Result.R = A.R * Color.R;
		Result.G = A.G * Color.G;
		Result.B = A.B * Color.B;
		return Result;
	}

	/** Division operator. */
	[[nodiscard]] friend inline TSHVectorRGB operator/(const TSHVectorRGB& A,const float& InB)
	{
		TSHVectorRGB Result;
		Result.R = A.R / InB;
		Result.G = A.G / InB;
		Result.B = A.B / InB;
		return Result;
	}

	/** Addition operator. */
	[[nodiscard]] friend inline TSHVectorRGB operator+(const TSHVectorRGB& A,const TSHVectorRGB& InB)
	{
		TSHVectorRGB Result;
		Result.R = A.R + InB.R;
		Result.G = A.G + InB.G;
		Result.B = A.B + InB.B;
		return Result;
	}
	
	/** Subtraction operator. */
	[[nodiscard]] friend inline TSHVectorRGB operator-(const TSHVectorRGB& A,const TSHVectorRGB& InB)
	{
		TSHVectorRGB Result;
		Result.R = A.R - InB.R;
		Result.G = A.G - InB.G;
		Result.B = A.B - InB.B;
		return Result;
	}

	/** Dot product operator. */
	[[nodiscard]] friend inline FLinearColor Dot(const TSHVectorRGB& A,const TSHVector<MaxSHOrder>& InB)
	{
		FLinearColor Result;
		Result.R = Dot(A.R,InB);
		Result.G = Dot(A.G,InB);
		Result.B = Dot(A.B,InB);
		Result.A = 1.0f;
		return Result;
	}

	/** In-place addition operator. */
	/** Changed from (*this = *this + InB;) to separate all calc to avoid LHS **/

	/** Now it calls directly += operator in TSHVector (avoid TSHVectorRGB + operator) **/
	inline TSHVectorRGB& operator+=(const TSHVectorRGB& InB)
	{
		R += InB.R;
		G += InB.G;
		B += InB.B;

		return *this;
	}
	
	/** In-place subtraction operator. */
	/** Changed from (*this = *this - InB;) to separate all calc to avoid LHS **/
	/** Now it calls directly -= operator in TSHVector (avoid TSHVectorRGB - operator) **/
	inline TSHVectorRGB& operator-=(const TSHVectorRGB& InB)
	{
		R -= InB.R;
		G -= InB.G;
		B -= InB.B;

		return *this;
	}

	/** In-place scalar multiplication operator. */
	/** Changed from (*this = *this * InB;) to separate all calc to avoid LHS **/
	/** Now it calls directly *= operator in TSHVector (avoid TSHVectorRGB * operator) **/
	inline TSHVectorRGB& operator*=(const float& Scalar)
	{
		R *= Scalar;
		G *= Scalar;
		B *= Scalar;

		return *this;
	}

	friend FArchive& operator<<(FArchive& Ar, TSHVectorRGB& SH)
	{
		return Ar << SH.R << SH.G << SH.B;
	}

	/** Adds an impulse to the SH environment. */
	inline void AddIncomingRadiance(const FLinearColor& IncomingRadiance, float Weight, const FVector4& WorldSpaceDirection)
	{
		*this += TSHVector<MaxSHOrder>::SHBasisFunction(WorldSpaceDirection) * (IncomingRadiance * Weight);
	}

	/** Adds ambient lighting. */
	inline void AddAmbient(const FLinearColor& Intensity)
	{
		*this += TSHVector<MaxSHOrder>::AmbientFunction() * Intensity;
	}
};

/** Color multiplication operator. */
template<int32 Order>
inline TSHVectorRGB<Order> operator*(const TSHVector<Order>& A,const FLinearColor& B)
{
	TSHVectorRGB<Order> Result;
	Result.R = A * B.R;
	Result.G = A * B.G;
	Result.B = A * B.B;

	return Result;
}

typedef TSHVector<3> FSHVector3;
typedef TSHVector<2> FSHVector2;
typedef TSHVectorRGB<3> FSHVectorRGB3;
typedef TSHVectorRGB<2> FSHVectorRGB2;
