// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __SCALAR_ISPH__
#define __SCALAR_ISPH__

// Define types
typedef double FReal;
typedef uint64 UIntType;
typedef int64 IntType;

#define REAL_BITS doublebits

// To make version checks easier, we combine the major and minor versions into a single number.
// We allocate three ditits for the minor version i.e. the format is going to be xyyy with x
// being the major version and yyy the minor version.
// We don't incorporate the patch version (ISPC_VERSION_PATCH from the compiler code) because
// it is not exposed to the *.ispc/h sources.
#define ISPC_VERSION		(ISPC_MAJOR_VERSION * 1000 + ISPC_MINOR_VERSION)

#if ISPC_VERSION >= 1024
	#undef  FLT_MAX
	#undef  FLT_MIN
#endif

#define FLOAT_PI					(3.1415926535897932f)
#define FLOAT_SMALL_NUMBER			(1.e-8f)
#define FLOAT_KINDA_SMALL_NUMBER	(1.e-4f)
#define FLOAT_SIGN_BIT				(0x80000000)
#define FLT_MAX						3.402823466e+38F
#define FLT_MIN						1.175494351e-38F
#define FLOAT_NON_FRACTIONAL		(8388608.f)
#define FLOAT_NAN					0xFFFFFFFF
#define FLOAT_ZERO					0.0f
#define FLOAT_ONE					1.0f
#define FLOAT_ONE_HALF				0.5f
#define FLOAT_TWO					2.0f

#if ISPC_VERSION >= 1024
	#undef  DBL_MAX
	#undef  DBL_MIN
#endif

#define DOUBLE_PI					(3.141592653589793238462643383279502884197169399d)
#define DOUBLE_SMALL_NUMBER			(1.d-8)
#define DOUBLE_KINDA_SMALL_NUMBER	(1.d-4)
#define DOUBLE_SIGN_BIT				(0x8000000000000000)
#define DBL_MAX						(1.7976931348623158d+308)
#define DBL_MIN						(2.2250738585072014d-308)
#define DOUBLE_NON_FRACTIONAL		(4503599627370496.0d)
#define DOUBLE_NAN					0xFFFFFFFFFFFFFFFF
#define DOUBLE_ZERO					0.0d
#define DOUBLE_ONE					1.0d
#define DOUBLE_ONE_HALF				0.5d
#define DOUBLE_TWO					2.0d

#define INDEX_NONE -1

#undef  PI
// Define type constants

#define PI						DOUBLE_PI
#define SMALL_NUMBER			DOUBLE_SMALL_NUMBER
#define KINDA_SMALL_NUMBER		DOUBLE_KINDA_SMALL_NUMBER
#define SIGN_BIT				DOUBLE_SIGN_BIT
#define REAL_MAX				DBL_MAX
#define REAL_MIN				DBL_MIN
#define NON_FRACTIONAL			DOUBLE_NON_FRACTIONAL
#define NAN						DOUBLE_NAN
#define ZERO					DOUBLE_ZERO
#define ONE_HALF				DOUBLE_ONE_HALF
#define ONE						DOUBLE_ONE
#define TWO						DOUBLE_TWO

#define QMULTI_SIGN_MASK0		DOUBLE_QMULTI_SIGN_MASK0
#define QMULTI_SIGN_MASK1		DOUBLE_QMULTI_SIGN_MASK1
#define QMULTI_SIGN_MASK2		DOUBLE_QMULTI_SIGN_MASK2
#define QINV_SIGN_MASK			DOUBLE_QINV_SIGN_MASK

#define DEG_TO_RAD				DOUBLE_DEG_TO_RAD
#define DEG_TO_RAD_HALF			DOUBLE_DEG_TO_RAD_HALF
#define RAD_TO_DEG				DOUBLE_RAD_TO_DEG

// _mm_cmpge_pd
inline uniform double VectorCompareGE(const uniform double A, const uniform double B)
{
	return A >= B ? doublebits(DOUBLE_NAN) : doublebits(0);
}

// _mm_cmpge_ps
inline uniform float VectorCompareGE(const uniform float A, const uniform float B)
{
	return A >= B ? floatbits(FLOAT_NAN) : floatbits(0);
}

// _mm_cmpgt_pd
inline varying double VectorCompareGT(const varying double A, const varying double B)
{
	return A > B ? doublebits(DOUBLE_NAN) : doublebits(0);
}

// _mm_cmpgt_ps
inline varying float VectorCompareGT(const varying float A, const varying float B)
{
	return A > B ? floatbits(FLOAT_NAN) : floatbits(0);
}

// _mm_cmple_pd
inline varying double VectorCompareLE(const varying double A, const varying double B)
{
	return A <= B ? doublebits(DOUBLE_NAN) : doublebits(0);
}

// _mm_cmple_ps
inline varying float VectorCompareLE(const varying float A, const varying float B)
{
	return A <= B ? floatbits(FLOAT_NAN) : floatbits(0);
}

// _mm_cmplt_pd
inline varying double VectorCompareLT(const varying double A, const varying double B)
{
	return A < B ? doublebits(DOUBLE_NAN) : doublebits(0);
}

// _mm_cmplt_ps
inline varying float VectorCompareLT(const varying float A, const varying float B)
{
	return A < B ? floatbits(FLOAT_NAN) : floatbits(0);
}

inline float InvSqrt(const float F)
{
	// Performs two passes of Newton-Raphson iteration on the hardware estimate
	//    v^-0.5 = x
	// => x^2 = v^-1
	// => 1/(x^2) = v
	// => F(x) = x^-2 - v
	//    F'(x) = -2x^-3

	//    x1 = x0 - F(x0)/F'(x0)
	// => x1 = x0 + 0.5 * (x0^-2 - Vec) * x0^3
	// => x1 = x0 + 0.5 * (x0 - Vec * x0^3)
	// => x1 = x0 + x0 * (0.5 - 0.5 * Vec * x0^2)
	//
	// This final form has one more operation than the legacy factorization (X1 = 0.5*X0*(3-(Y*X0)*X0)
	// but retains better accuracy (namely InvSqrt(1) = 1 exactly).

	static const uniform float fOneHalf = 0.5f;
	float X0, X1, X2, FOver2;

	X0 = rsqrt_fast(F);	// 1/sqrt estimate (12 bits)
	FOver2 = F * fOneHalf;

	// 1st Newton-Raphson iteration
	X1 = X0 * X0;
	X1 = fOneHalf - (FOver2 * X1);
	X1 = X0 + (X0 * X1);

	// 2nd Newton-Raphson iteration
	X2 = X1 * X1;
	X2 = fOneHalf - (FOver2 * X2);
	X2 = X1 + (X1 * X2);

	return X2;
}

inline double InvSqrt(const double F)
{
	return 1 / sqrt(F);
}

inline uniform float InvSqrt(const uniform float F)
{
	// Performs two passes of Newton-Raphson iteration on the hardware estimate
	//    v^-0.5 = x
	// => x^2 = v^-1
	// => 1/(x^2) = v
	// => F(x) = x^-2 - v
	//    F'(x) = -2x^-3

	//    x1 = x0 - F(x0)/F'(x0)
	// => x1 = x0 + 0.5 * (x0^-2 - Vec) * x0^3
	// => x1 = x0 + 0.5 * (x0 - Vec * x0^3)
	// => x1 = x0 + x0 * (0.5 - 0.5 * Vec * x0^2)
	//
	// This final form has one more operation than the legacy factorization (X1 = 0.5*X0*(3-(Y*X0)*X0)
	// but retains better accuracy (namely InvSqrt(1) = 1 exactly).

	static const uniform float fOneHalf = 0.5f;
	uniform float X0, X1, X2, FOver2;

	X0 = rsqrt_fast(F);	// 1/sqrt estimate (12 bits)
	FOver2 = F * fOneHalf;

	// 1st Newton-Raphson iteration
	X1 = X0 * X0;
	X1 = fOneHalf - (FOver2 * X1);
	X1 = X0 + (X0 * X1);

	// 2nd Newton-Raphson iteration
	X2 = X1 * X1;
	X2 = fOneHalf - (FOver2 * X2);
	X2 = X1 + (X1 * X2);

	return X2;
}

inline uniform double InvSqrt(const uniform double F)
{
	return 1 / sqrt(F);
}

inline uniform float ReciprocalSqrtAccurate(const uniform float V)
{
	// Perform two passes of Newton-Raphson iteration on the hardware estimate
	//    v^-0.5 = x
	// => x^2 = v^-1
	// => 1/(x^2) = v
	// => F(x) = x^-2 - v
	//    F'(x) = -2x^-3

	//    x1 = x0 - F(x0)/F'(x0)
	// => x1 = x0 + 0.5 * (x0^-2 - Vec) * x0^3
	// => x1 = x0 + 0.5 * (x0 - Vec * x0^3)
	// => x1 = x0 + x0 * (0.5 - 0.5 * Vec * x0^2)

	const uniform float DivBy2 = V * 0.5f;

	// Initial estimate
	const uniform float x0 = rsqrt_fast(V);

	// First iteration
	uniform float x1 = x0 * x0;
	x1 = 0.5f - (DivBy2 * x1);
	x1 = x0 * x1 + x0;

	// Second iteration
	uniform float x2 = x1 * x1;
	x2 = 0.5f - (DivBy2 * x2);
	x2 = x1 * x2 + x1;

	return x2;
}

inline float ReciprocalSqrtAccurate(const float V)
{
	// Perform two passes of Newton-Raphson iteration on the hardware estimate
	//    v^-0.5 = x
	// => x^2 = v^-1
	// => 1/(x^2) = v
	// => F(x) = x^-2 - v
	//    F'(x) = -2x^-3

	//    x1 = x0 - F(x0)/F'(x0)
	// => x1 = x0 + 0.5 * (x0^-2 - Vec) * x0^3
	// => x1 = x0 + 0.5 * (x0 - Vec * x0^3)
	// => x1 = x0 + x0 * (0.5 - 0.5 * Vec * x0^2)

	const float DivBy2 = V * 0.5f;

	// Initial estimate
	const float x0 = rsqrt_fast(V);

	// First iteration
	float x1 = x0 * x0;
	x1 = 0.5f - (DivBy2 * x1);
	x1 = x0 * x1 + x0;

	// Second iteration
	float x2 = x1 * x1;
	x2 = 0.5f - (DivBy2 * x2);
	x2 = x1 * x2 + x1;

	return x2;
}

inline uniform double ReciprocalSqrtAccurate(const uniform double V)
{
	return 1.0d / sqrt(V);
}

inline double ReciprocalSqrtAccurate(const double V)
{
	return 1.0d / sqrt(V);
}

// _mm_and_pd
inline varying double VectorBitwiseAnd(const varying double A, const varying double B)
{
	return doublebits(intbits(A) & intbits(B));
}

// _mm_and_ps
inline varying float VectorBitwiseAnd(const varying float A, const varying float B)
{
	return floatbits(intbits(A) & intbits(B));
}

// _mm_or_pd
inline varying double VectorBitwiseOr(const varying double A, const varying double B)
{
	return doublebits(intbits(A) | intbits(B));
}

// _mm_or_ps
inline varying float VectorBitwiseOr(const varying float A, const varying float B)
{
	return floatbits(intbits(A) | intbits(B));
}

// _mm_xor_pd
inline varying double VectorBitwiseXor(const varying double A, const varying double B)
{
	return doublebits(intbits(A) ^ intbits(B));
}

// _mm_xor_ps
inline varying float VectorBitwiseXor(const varying float A, const varying float B)
{
	return floatbits(intbits(A) ^ intbits(B));
}

inline varying double VectorSelect(const varying double Mask, const varying double A, const varying double B)
{
	return VectorBitwiseXor(B, VectorBitwiseAnd(Mask, VectorBitwiseXor(A, B)));
}

inline varying float VectorSelect(const varying float Mask, const varying float A, const varying float B)
{
	return VectorBitwiseXor(B, VectorBitwiseAnd(Mask, VectorBitwiseXor(A, B)));
}

inline varying float VectorSwizzle(const varying float A, const uniform int X, const uniform int Y, const uniform int Z, const uniform int W)
{
#if TARGET_WIDTH == 4
	const varying int vPerm = { X, Y, Z, W };
#elif TARGET_WIDTH == 8
	const varying int vPerm = { X, Y, Z, W, X+4, Y+4, Z+4, W+4 };
#elif TARGET_WIDTH == 16
	const varying int vPerm = { X, Y, Z, W, X+4, Y+4, Z+4, W+4, X+8, Y+8, Z+8, W+8, X+12, Y+12, Z+12, W+12 };
#endif

	return shuffle(A, vPerm);
}

inline varying double VectorSwizzle(const varying double A, const uniform int X, const uniform int Y, const uniform int Z, const uniform int W)
{
#if TARGET_WIDTH == 4
	const varying int vPerm = { X, Y, Z, W };
#elif TARGET_WIDTH == 8
	const varying int vPerm = { X, Y, Z, W, X+4, Y+4, Z+4, W+4 };
#elif TARGET_WIDTH == 16
	const varying int vPerm = { X, Y, Z, W, X+4, Y+4, Z+4, W+4, X+8, Y+8, Z+8, W+8, X+12, Y+12, Z+12, W+12 };
#endif

	return shuffle(A, vPerm);
}

#endif
