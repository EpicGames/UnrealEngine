// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Math/VectorRegister.h"

/**
* This define controls whether a scalar implementation or vector implementation is used for FTransform.
* The vector implementation works even when using UnMathFPU, but it will be much slower than the equivalent
* scalar implementation, so the scalar code is maintained and enabled when vector intrinsics are off.
*/
//Currently disabled because FBoneAtom became FTransform and we want to iterate quickly on it.
#ifndef ENABLE_VECTORIZED_TRANSFORM
#define ENABLE_VECTORIZED_TRANSFORM		PLATFORM_ENABLE_VECTORINTRINSICS
#endif

#if ENABLE_VECTORIZED_TRANSFORM

/**
* The ScalarRegister class wraps the concept of a 'float-in-vector', allowing common scalar operations like bone
* weight calculations to be done in vector registers.  This will avoid some LHS hazards that arise when mixing float
* and vector math on some platforms.  However, doing the math for four elements is slower if the vector operations are
* being emulated on a scalar FPU, so ScalarRegister is defined to float when ENABLE_VECTORIZED_TRANSFORM == 0.
*/
class ScalarRegister
{
public:
	using RegisterType = VectorRegister4Float;

	RegisterType Value;

	/** default constructor */
	[[nodiscard]] UE_FORCEINLINE_HINT ScalarRegister();

	/** Copy Constructor */
	[[nodiscard]] UE_FORCEINLINE_HINT ScalarRegister(const ScalarRegister& VectorValue);

	/** Constructor using float value */
	[[nodiscard]] explicit UE_FORCEINLINE_HINT ScalarRegister(const float& ScalarValue);

	/** Constructor
	 *
	 * @param RegisterType float4 vector register type
	 */
	[[nodiscard]] explicit UE_FORCEINLINE_HINT ScalarRegister(RegisterType VectorValue);

	/**
	 * Gets the result of multiplying a scalar register to this.
	 *
	 * @param RHS The scalar register to multiply this by.
	 *
	 * @return The result of multiplication.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT ScalarRegister operator*(const ScalarRegister& RHS) const;

	/**
	 * Gets the result of adding a scalar register to this.
	 *
	 * @param RHS The scalar register to add.
	 *
	 * @return The result of addition.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT ScalarRegister operator+(const ScalarRegister& RHS) const;

	/**
	 * Adds to this scalar register.
	 *
	 * @param RHS The scalar register to add to this.
	 *
	 * @return Reference to this after addition.
	 */
	inline ScalarRegister& operator+=(const ScalarRegister& RHS);

	/**
	 * Subtracts another scalar register from this.
	 *
	 * @param RHS The other scalar register.
	 *
	 * @return reference to this after subtraction.
	 */
	inline ScalarRegister& operator-=(const ScalarRegister& RHS);

	/**
	 * Gets the result of subtracting a scalar register to this.
	 *
	 * @param RHS The scalar register to subtract.
	 *
	 * @return The result of subtraction.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT ScalarRegister operator-(const ScalarRegister& RHS) const;

	
	/** assignment operator
	 *
	 * @param RHS a ScalarRegister
	 */
	inline ScalarRegister& operator=(const ScalarRegister& RHS);

	/** assignment operator
	 *
	 * @param RHS a RegisterType
	 */
	inline ScalarRegister& operator=(const RegisterType& RHS);

	/**
	 * ScalarRegister to RegisterType conversion operator.
	 */
	UE_FORCEINLINE_HINT operator RegisterType() const;
};

UE_FORCEINLINE_HINT ScalarRegister::ScalarRegister()
{
}

UE_FORCEINLINE_HINT ScalarRegister::ScalarRegister(const ScalarRegister& VectorValue)
{
	Value = VectorValue.Value;
}

UE_FORCEINLINE_HINT ScalarRegister::ScalarRegister(const float& ScalarValue)
{
	Value = VectorLoadFloat1(&ScalarValue);
}

UE_FORCEINLINE_HINT ScalarRegister::ScalarRegister(RegisterType VectorValue)
{
	Value = VectorValue;
}

UE_FORCEINLINE_HINT ScalarRegister ScalarRegister::operator*(const ScalarRegister& RHS) const
{
	return ScalarRegister(VectorMultiply(Value, RHS.Value));
}

UE_FORCEINLINE_HINT ScalarRegister ScalarRegister::operator+(const ScalarRegister& RHS) const
{
	return ScalarRegister(VectorAdd(Value, RHS.Value));
}

inline ScalarRegister& ScalarRegister::operator+=(const ScalarRegister& RHS)
{
	Value = VectorAdd(Value, RHS.Value);
	return *this;
}

inline ScalarRegister& ScalarRegister::operator-=(const ScalarRegister& RHS)
{
	Value = VectorSubtract(Value, RHS.Value);
	return *this;
}

UE_FORCEINLINE_HINT ScalarRegister ScalarRegister::operator-(const ScalarRegister& RHS) const
{
	return ScalarRegister(VectorSubtract(Value, RHS.Value));
}

	
inline ScalarRegister& ScalarRegister::operator=(const ScalarRegister& RHS)
{
	Value = RHS.Value;
	return *this;
}

inline ScalarRegister& ScalarRegister::operator=(const RegisterType& RHS)
{
	Value = RHS;
	return *this;
}

UE_FORCEINLINE_HINT ScalarRegister::operator RegisterType() const
{
	return Value;
}

#define ScalarOne (ScalarRegister)ScalarRegister(VectorOne())
#define ScalarZero (ScalarRegister)ScalarRegister(VectorZero())

/*----------------------------------------------------------------------------
	ScalarRegister specialization of templates.
----------------------------------------------------------------------------*/

/** Returns the smaller of the two values */
UE_FORCEINLINE_HINT ScalarRegister ScalarMin(const ScalarRegister& A, const ScalarRegister& B)
{
	return ScalarRegister(VectorMin(A.Value, B.Value));
}

/** Returns the larger of the two values */
UE_FORCEINLINE_HINT ScalarRegister ScalarMax(const ScalarRegister& A, const ScalarRegister& B)
{
	return ScalarRegister(VectorMax(A.Value, B.Value));
}

// TCustomLerp for FMath::Lerp()
template<> struct TCustomLerp<ScalarRegister>
{
	constexpr static bool Value = true;

	// Specialization of Lerp template that works with scalar (float in vector) registers
	inline ScalarRegister Lerp(const ScalarRegister& A, const ScalarRegister& B, const ScalarRegister& Alpha)
	{
		const ScalarRegister::RegisterType Delta = VectorSubtract(B.Value, A.Value);
		return ScalarRegister(VectorMultiplyAdd(Alpha.Value, Delta, A.Value));
	}
};

/**
 * Computes the reciprocal of the scalar register (component-wise) and returns the result.
 *
 * @param A	1st scalar
 * @return		ScalarRegister( 1.0f / A.x, 1.0f / A.y, 1.0f / A.z, 1.0f / A.w )
 */
UE_FORCEINLINE_HINT ScalarRegister ScalarReciprocal(const ScalarRegister& A)
{
	return ScalarRegister(VectorReciprocalAccurate(A.Value));
}

/**
 * Returns zero if any element in A is greater than the corresponding element in the global AnimWeightThreshold.
 *
 * @param A 1st source vector
 * @return zero integer if (A.x > AnimWeightThreshold.x) || (A.y > AnimWeightThreshold.y) || (A.z > AnimWeightThreshold.z) || (A.w > AnimWeightThreshold.w), non-zero Otherwise
 */
#define NonZeroAnimWeight(A) VectorAnyGreaterThan(A.Value, GlobalVectorConstants::AnimWeightThreshold)


/**
 * Returns non-zero if any element in A is greater than the corresponding element in the global AnimWeightThreshold.
 *
 * @param A 1st source vector
 * @return Non-zero integer if (A.x > AnimWeightThreshold.x) || (A.y > AnimWeightThreshold.y) || (A.z > AnimWeightThreshold.z) || (A.w > AnimWeightThreshold.w)
 */
#define NonOneAnimWeight(A) !VectorAnyGreaterThan(A.Value, VectorSubtract(VectorOne(), GlobalVectorConstants::AnimWeightThreshold))

#else

#define ScalarRegister float

#define ScalarOne 1.0f
#define ScalarZero 0.0f

#define ScalarMin Min
#define ScalarMax Max

#define ScalarReciprocal(A)  (1.0f / (A))

#define NonZeroAnimWeight(A) ((A) > ZERO_ANIMWEIGHT_THRESH)
#define NonOneAnimWeight(A) ((A) < 1.0f - ZERO_ANIMWEIGHT_THRESH)

#endif
