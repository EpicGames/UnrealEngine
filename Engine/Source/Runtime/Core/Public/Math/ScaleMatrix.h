// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Plane.h"
#include "Math/Matrix.h"

namespace UE {
namespace Math {

/**
 * Scale matrix.
 */
template<typename T>
struct TScaleMatrix
	: public TMatrix<T>
{
public:

	/**
	 * @param Scale uniform scale to apply to matrix.
	 */
	[[nodiscard]] TScaleMatrix( T Scale );

	/**
	 * @param Scale Non-uniform scale to apply to matrix.
	 */
	[[nodiscard]] TScaleMatrix( const TVector<T>& Scale );

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	[[nodiscard]] explicit TScaleMatrix(const TScaleMatrix<FArg>& From)
		: TMatrix<T>(From)
	{
	}
	
	/** Matrix factory. Return an TMatrix<T> so we don't have type conversion issues in expressions. */
	[[nodiscard]] static TMatrix<T> Make(T Scale)
	{
		return TScaleMatrix<T>(Scale);
	}

	/** Matrix factory. Return an TMatrix<T> so we don't have type conversion issues in expressions. */
	[[nodiscard]] static TMatrix<T> Make(const TVector<T>& Scale)
	{
		return TScaleMatrix<T>(Scale);
	}
};


/* FScaleMatrix inline functions
 *****************************************************************************/

template<typename T>
inline TScaleMatrix<T>::TScaleMatrix( T Scale )
	: TMatrix<T>(
		TPlane<T>(Scale,	0.0f,	0.0f,	0.0f),
		TPlane<T>(0.0f,		Scale,	0.0f,	0.0f),
		TPlane<T>(0.0f,		0.0f,	Scale,	0.0f),
		TPlane<T>(0.0f,		0.0f,	0.0f,	1.0f)
	)
{ }


template<typename T>
inline TScaleMatrix<T>::TScaleMatrix( const TVector<T>& Scale )
	: TMatrix<T>(
		TPlane<T>(Scale.X,	0.0f,		0.0f,		0.0f),
		TPlane<T>(0.0f,		Scale.Y,	0.0f,		0.0f),
		TPlane<T>(0.0f,		0.0f,		Scale.Z,	0.0f),
		TPlane<T>(0.0f,		0.0f,		0.0f,		1.0f)
	)
{ }
	
} // namespace Math
} // namespace UE

UE_DECLARE_LWC_TYPE(ScaleMatrix, 44);

template<> struct TIsUECoreVariant<FScaleMatrix44f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FScaleMatrix44d> { enum { Value = true }; };