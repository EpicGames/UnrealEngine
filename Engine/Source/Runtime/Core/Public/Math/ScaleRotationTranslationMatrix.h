// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Matrix.h"


namespace UE {
namespace Math {
		
/** Combined Scale rotation and translation matrix */
template<typename T>
struct TScaleRotationTranslationMatrix
	: public TMatrix<T>
{
public:
	using TMatrix<T>::M;
	
	/**
	 * Constructor.
	 *
	 * @param Scale scale to apply to matrix
	 * @param Rot rotation
	 * @param Origin translation to apply
	 */
	TScaleRotationTranslationMatrix(const TVector<T>& Scale, const TRotator<T>& Rot, const TVector<T>& Origin);
	
	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TScaleRotationTranslationMatrix(const TScaleRotationTranslationMatrix<FArg>& From)
		: TMatrix<T>(From)
	{
	}
};

namespace
{
	template<typename T>
	void GetSinCos(T& S, T& C, T Degrees)
	{
		if (Degrees == 0.f)
		{
			S = 0.f;
			C = 1.f;
		}
		else if (Degrees == 90.f)
		{
			S = 1.f;
			C = 0.f;
		}
		else if (Degrees == 180.f)
		{
			S = 0.f;
			C = -1.f;
		}
		else if (Degrees == 270.f)
		{
			S = -1.f;
			C = 0.f;
		}
		else
		{
			FMath::SinCos(&S, &C, FMath::DegreesToRadians(Degrees));
		}
	}
}

template<typename T>
inline TScaleRotationTranslationMatrix<T>::TScaleRotationTranslationMatrix(const TVector<T>& Scale, const TRotator<T>& Rot, const TVector<T>& Origin)
{
	T SP, SY, SR;
	T CP, CY, CR;
	GetSinCos(SP, CP, (T)Rot.Pitch);
	GetSinCos(SY, CY, (T)Rot.Yaw);
	GetSinCos(SR, CR, (T)Rot.Roll);

	M[0][0]	= (CP * CY) * Scale.X;
	M[0][1]	= (CP * SY) * Scale.X;
	M[0][2]	= (SP) * Scale.X;
	M[0][3]	= 0.f;

	M[1][0]	= (SR * SP * CY - CR * SY) * Scale.Y;
	M[1][1]	= (SR * SP * SY + CR * CY) * Scale.Y;
	M[1][2]	= (- SR * CP) * Scale.Y;
	M[1][3]	= 0.f;

	M[2][0]	= ( -( CR * SP * CY + SR * SY ) ) * Scale.Z;
	M[2][1]	= (CY * SR - CR * SP * SY) * Scale.Z;
	M[2][2]	= (CR * CP) * Scale.Z;
	M[2][3]	= 0.f;

	M[3][0]	= Origin.X;
	M[3][1]	= Origin.Y;
	M[3][2]	= Origin.Z;
	M[3][3]	= 1.f;
}
	
} // namespace Math
} // namespace UE

UE_DECLARE_LWC_TYPE(ScaleRotationTranslationMatrix, 44);

template<> struct TIsUECoreVariant<FScaleRotationTranslationMatrix44f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FScaleRotationTranslationMatrix44d> { enum { Value = true }; };