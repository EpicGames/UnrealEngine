// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __SOAAOS_ISPH__
#define __SOAAOS_ISPH__

// Change layout from
// input a0 a1 ... aX b0 b1 ... bX c0 c1 ... cX d0 d1 .. dX aX+1 ...
// to
// output: a0 b0 c0 d0 a1 b1 c1 d1 a2 b2 c2 d2 ...

#if TARGET_WIDTH == 4
static const varying int vSOA20 = {0, 4, 1, 5};
static const varying int vSOA21 = {2, 6, 3, 7};
#elif TARGET_WIDTH == 8
static const varying int vSOA20 = {0, 8, 1, 9, 2, 10, 3, 11};
static const varying int vSOA21 = {4, 12, 5, 13, 6, 14, 7, 15};
#elif TARGET_WIDTH == 16
static const varying int vSOA20 = {0, 16, 1, 17, 2, 18, 3, 19, 4, 20, 5, 21, 6, 22, 7, 23};
static const varying int vSOA21 = {8, 24, 9, 25, 10, 26, 11, 27, 12, 28, 13, 29, 14, 30, 15, 31};
#else
#error "No implementation for this target"
#endif

template<typename T>
unmasked inline void soa_to_aos2_ispc(const varying T &src0, const varying T &src1, uniform T dst[])
{
    dst[programIndex] = shuffle(src0, src1, vSOA20);
    dst[programIndex + programCount] = shuffle(src0, src1, vSOA21);
}

#if TARGET_WIDTH == 4
static const varying int vSOA30 = {0, 1, 2, 4};
static const varying int vSOA31 = {1, 2, 4, 5};
static const varying int vSOA32 = {0, 3, 6, 1};
static const varying int vSOA33 = {4, 7, 2, 5};
static const varying int vSOA34 = {3, 7, -1, -1};
static const varying int vSOA35 = {6, 0, 1, 7};
#elif TARGET_WIDTH == 8
static const varying int vSOA30 = {0, 1, 2, 3, 4, 5, 8, 9};
static const varying int vSOA31 = {2, 3, 4, 8, 9, 10, 11, 12};
static const varying int vSOA32 = {0, 6, 11, 1, 7, 12, 2, 8};
static const varying int vSOA33 = {13, 3, 9, 14, 4, 10, 15, 5};
static const varying int vSOA34 = {6, 7, 13, 14, 15, -1, -1, -1};
static const varying int vSOA35 = {2, 13, 0, 3, 14, 1, 4, 15};
#elif TARGET_WIDTH == 16
static const varying int vSOA30 = {0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 16, 17, 18, 19, 20};
static const varying int vSOA31 = {5, 6, 7, 8, 9, 10, 16, 17, 18, 19, 20, 21, 22, 23, 24, 25};
static const varying int vSOA32 = {0, 11, 22, 1, 12, 23, 2, 13, 24, 3, 14, 25, 4, 15, 26, 5};
static const varying int vSOA33 = {16, 27, 6, 17, 28, 7, 18, 29, 8, 19, 30, 9, 20, 31, 10, 21};
static const varying int vSOA34 = {11, 12, 13, 14, 15, 27, 28, 29, 30, 31, -1, -1, -1, -1, -1, -1};
static const varying int vSOA35 = {26, 0, 5, 27, 1, 6, 28, 2, 7, 29, 3, 8, 30, 4, 9, 31};
#else
#error "No implementation for this target"
#endif

template<typename T>
unmasked inline void soa_to_aos3_ispc(const varying T &src0, const varying T &src1, const varying T &src2, uniform T dst[])
{
    const varying T s0 = src0;
    const varying T s1 = src1;
    const varying T s2 = src2;
                              
    const varying T t0 = shuffle(s0, s1, vSOA30);
    const varying T t1 = shuffle(s1, s2, vSOA31);
    dst[programIndex] = shuffle(t0, t1, vSOA32);
    dst[programCount + programIndex] = shuffle(t0, t1, vSOA33);

    const varying T t2 = shuffle(s0, s1, vSOA34);
    dst[2 * programCount + programIndex] = shuffle(t2, s2, vSOA35);
}

#if TARGET_WIDTH == 4
static const varying int vSOA40 = {0, 1, 4, 5};
static const varying int vSOA41 = {2, 3, 6, 7};
static const varying int vSOA42 = {0, 4, 1, 5};
static const varying int vSOA43 = {2, 6, 3, 7};
#elif TARGET_WIDTH == 8
static const varying int vSOA40 = {0, 8, 1, 9, 4, 12, 5, 13};
static const varying int vSOA41 = {2, 10, 3, 11, 6, 14, 7, 15};
static const varying int vSOA42 = {0, 1, 8, 9, 4, 5, 12, 13};
static const varying int vSOA43 = {2, 3, 10, 11, 6, 7, 14, 15};
static const varying int vSOA44 = {0, 1, 2, 3, 8, 9, 10, 11};
static const varying int vSOA45 = {4, 5, 6, 7, 12, 13, 14, 15};

static const varying int vSOA401 = {0, 1, 8, 9, 4, 5, 12, 13};
static const varying int vSOA411 = {2, 3, 10, 11, 6, 7, 14, 15};
static const varying int vSOA421 = {0, 1, 2, 3, 8, 9, 10, 11};
static const varying int vSOA431 = {4, 5, 6, 7, 12, 13, 14, 15};
#elif TARGET_WIDTH == 16
static const varying int vSOA40 = {0, 8, 16, 24, 1, 9, 17, 25, 2, 10, 18, 26, 3, 11, 19, 27};
static const varying int vSOA41 = {4, 12, 20, 28, 5, 13, 21, 29, 6, 14, 22, 30, 7, 15, 23, 31};
static const varying int vSOA42 = {0, 1, 2, 3, 4, 5, 6, 7, 16, 17, 18, 19, 20, 21, 22, 23};
static const varying int vSOA43 = {8, 9, 10, 11, 12, 13, 14, 15, 24, 25, 26, 27, 28, 29, 30, 31};
#else
#error "No implementation for this target"
#endif

#if TARGET_WIDTH == 8
template<typename T>
unmasked inline void soa_to_aos4_ispc(const varying T& src0, const varying T& src1,
    const varying T& src2, const varying T& src3, uniform T dst[])
{
    const varying T t0 = shuffle(src0, src1, vSOA40);
    const varying T t1 = shuffle(src2, src3, vSOA40);
    const varying T t2 = shuffle(t0, t1, vSOA42);
    const varying T t3 = shuffle(t0, t1, vSOA43);

    const varying T t4 = shuffle(src0, src1, vSOA41);
    const varying T t5 = shuffle(src2, src3, vSOA41);
    const varying T t6 = shuffle(t4, t5, vSOA42);
    const varying T t7 = shuffle(t4, t5, vSOA43);

    dst[programIndex] = shuffle(t2, t3, vSOA44);
    dst[programCount + programIndex] = shuffle(t6, t7, vSOA44);
    dst[2 * programCount + programIndex] = shuffle(t2, t3, vSOA45);
    dst[3 * programCount + programIndex] = shuffle(t6, t7, vSOA45);
}

template<>
unmasked inline void soa_to_aos4_ispc<double>(const varying double& src0, const varying double& src1,
    const varying double& src2, const varying double& src3, uniform double dst[])
{
    uniform float *uniform dstflt = (uniform float *uniform)dst;
    const uniform double<TARGET_WIDTH> x0 = *((const uniform double<TARGET_WIDTH>* uniform)&src0);
    const uniform double<TARGET_WIDTH> x1 = *((const uniform double<TARGET_WIDTH>* uniform)&src1);
    const uniform double<TARGET_WIDTH> x2 = *((const uniform double<TARGET_WIDTH>* uniform)&src2);
    const uniform double<TARGET_WIDTH> x3 = *((const uniform double<TARGET_WIDTH>* uniform)&src3);

    const varying float u0 = *((varying float *uniform)&x0);
    const varying float u1 = *((varying float *uniform)&x0[TARGET_WIDTH/2]);
    const varying float u2 = *((varying float *uniform)&x1);
    const varying float u3 = *((varying float *uniform)&x1[TARGET_WIDTH/2]);
    const varying float u4 = *((varying float *uniform)&x2);
    const varying float u5 = *((varying float *uniform)&x2[TARGET_WIDTH/2]);
    const varying float u6 = *((varying float *uniform)&x3);
    const varying float u7 = *((varying float *uniform)&x3[TARGET_WIDTH/2]);

    const varying float t0 = shuffle(u0, u2, vSOA401);
    const varying float t1 = shuffle(u0, u2, vSOA411);
    const varying float t2 = shuffle(u1, u3, vSOA401);
    const varying float t3 = shuffle(u1, u3, vSOA411);
    const varying float t4 = shuffle(u4, u6, vSOA401);
    const varying float t5 = shuffle(u4, u6, vSOA411);
    const varying float t6 = shuffle(u5, u7, vSOA401);
    const varying float t7 = shuffle(u5, u7, vSOA411);

    dstflt[programIndex] = shuffle(t0, t4, vSOA421);
    dstflt[programCount + programIndex] = shuffle(t1, t5, vSOA421);
    dstflt[2 * programCount + programIndex] = shuffle(t0, t4, vSOA431);
    dstflt[3 * programCount + programIndex] = shuffle(t1, t5, vSOA431);
    dstflt[4 * programCount + programIndex] = shuffle(t2, t6, vSOA421);
    dstflt[5 * programCount + programIndex] = shuffle(t3, t7, vSOA421);
    dstflt[6 * programCount + programIndex] = shuffle(t2, t6, vSOA431);
    dstflt[7 * programCount + programIndex] = shuffle(t3, t7, vSOA431);
}

template<>
unmasked inline void soa_to_aos4_ispc<int64>(const varying int64& src0, const varying int64& src1,
    const varying int64& src2, const varying int64& src3, uniform int64 dst[])
{
    uniform float* uniform dstflt = (uniform float* uniform)dst;
    const uniform int64<TARGET_WIDTH>x0 = *((const uniform int64<TARGET_WIDTH>* uniform)&src0);
    const uniform int64<TARGET_WIDTH>x1 = *((const uniform int64<TARGET_WIDTH>* uniform)&src1);
    const uniform int64<TARGET_WIDTH>x2 = *((const uniform int64<TARGET_WIDTH>* uniform)&src2);
    const uniform int64<TARGET_WIDTH>x3 = *((const uniform int64<TARGET_WIDTH>* uniform)&src3);

    const varying float u0 = *((varying float* uniform)&x0);
    const varying float u1 = *((varying float* uniform)&x0[TARGET_WIDTH / 2]);
    const varying float u2 = *((varying float* uniform)&x1);
    const varying float u3 = *((varying float* uniform)&x1[TARGET_WIDTH / 2]);
    const varying float u4 = *((varying float* uniform)&x2);
    const varying float u5 = *((varying float* uniform)&x2[TARGET_WIDTH / 2]);
    const varying float u6 = *((varying float* uniform)&x3);
    const varying float u7 = *((varying float* uniform)&x3[TARGET_WIDTH / 2]);

    const varying float t0 = shuffle(u0, u2, vSOA401);
    const varying float t1 = shuffle(u0, u2, vSOA411);
    const varying float t2 = shuffle(u1, u3, vSOA401);
    const varying float t3 = shuffle(u1, u3, vSOA411);
    const varying float t4 = shuffle(u4, u6, vSOA401);
    const varying float t5 = shuffle(u4, u6, vSOA411);
    const varying float t6 = shuffle(u5, u7, vSOA401);
    const varying float t7 = shuffle(u5, u7, vSOA411);

    dstflt[programIndex] = shuffle(t0, t4, vSOA421);
    dstflt[programCount + programIndex] = shuffle(t1, t5, vSOA421);
    dstflt[2 * programCount + programIndex] = shuffle(t0, t4, vSOA431);
    dstflt[3 * programCount + programIndex] = shuffle(t1, t5, vSOA431);
    dstflt[4 * programCount + programIndex] = shuffle(t2, t6, vSOA421);
    dstflt[5 * programCount + programIndex] = shuffle(t3, t7, vSOA421);
    dstflt[6 * programCount + programIndex] = shuffle(t2, t6, vSOA431);
    dstflt[7 * programCount + programIndex] = shuffle(t3, t7, vSOA431);
}

#else
template<typename T>
unmasked inline void soa_to_aos4_ispc(const varying T &src0, const varying T &src1,
    const varying T &src2, const varying T &src3, uniform T dst[])
{
    const varying T t0 = shuffle(src0, src1, vSOA42);
    const varying T t1 = shuffle(src2, src3, vSOA42);
    const varying T t2 = shuffle(src0, src1, vSOA43);
    const varying T t3 = shuffle(src2, src3, vSOA43);

    dst[programIndex] = shuffle(t0, t1, vSOA40);
    dst[programCount + programIndex] = shuffle(t0, t1, vSOA41);
    dst[2 * programCount + programIndex] = shuffle(t2, t3, vSOA40);
    dst[3 * programCount + programIndex] = shuffle(t2, t3, vSOA41);
}
#endif

#if TARGET_WIDTH == 4
static const varying int vSOA60 = {0, 1, 4, 5};
static const varying int vSOA61 = {2, 3, 6, 7};
static const varying int vSOA62 = {0, 2, 4, 6};
static const varying int vSOA63 = {1, 3, 4, 7};
static const varying int vSOA64 = {0, 2, 5, 7};
#elif TARGET_WIDTH == 8
static const varying int vSOA60 = {0, 1, 2, 3, 8, 9, 10, 11};
static const varying int vSOA61 = {4, 5, 6, 7, 12, 13, 14, 15};
static const varying int vSOA62 = {0, 4, 8, 12, -1, -1, 1, 5};
static const varying int vSOA63 = {0, 1, 2, 3, 8, 12, 6, 7};
static const varying int vSOA64 = {9, 13, -1, -1, 2, 6, 10, 14};
static const varying int vSOA65 = {0, 1, 9, 13, 4, 5, 6, 7};
static const varying int vSOA66 = {-1, -1, 3, 7, 11, 15, -1, -1};
static const varying int vSOA67 = {10, 14, 2, 3, 4, 5, 11, 15};
#elif TARGET_WIDTH == 16
static const varying int vSOA60 = {0, 1, 2, 3, 4, 5, 6, 7, 16, 17, 18, 19, 20, 21, 22, 23};
static const varying int vSOA61 = {8, 9, 10, 11, 12, 13, 14, 15, 24, 25, 26, 27, 28, 29, 30, 31};
static const varying int vSOA62 = {0, 8, 16, 24, -1, -1, 1, 9, 17, 25, -1, -1, 2, 10, 18, 26};
static const varying int vSOA63 = {0, 1, 2, 3, 16, 24, 6, 7, 8, 9, 17, 25, 12, 13, 14, 15};
static const varying int vSOA64 = {-1, -1, 3, 11, 19, 27, -1, -1, 4, 12, 20, 28, -1, -1, 5, 13};
static const varying int vSOA65 = {18, 26, 2, 3, 4, 5, 19, 27, 8, 9, 10, 11, 20, 28, 14, 15};
static const varying int vSOA66 = {21, 29, -1, -1, 6, 14, 22, 30, -1, -1, 7, 15, 23, 31, -1, -1};
static const varying int vSOA67 = {0, 1, 21, 29, 4, 5, 6, 7, 22, 30, 10, 11, 12, 13, 23, 31};
#else
#error "No implementation for this target"
#endif

#if TARGET_WIDTH == 4
template<typename T>
unmasked inline void soa_to_aos6_ispc(const varying T &src0, const varying T &src1, const varying T &src2, const varying T &src3, const varying T &src4, const varying T &src5, uniform T dst[])
{
    const varying T t0 = shuffle(src0, src1, vSOA60);
    const varying T t1 = shuffle(src0, src1, vSOA61);
    const varying T t2 = shuffle(src2, src3, vSOA60);
    const varying T t3 = shuffle(src2, src3, vSOA61);
    const varying T t4 = shuffle(src4, src5, vSOA60);
    const varying T t5 = shuffle(src4, src5, vSOA61);

    dst[programIndex] = shuffle(t0, t2, vSOA62);
    dst[programIndex + programCount] = shuffle(t4, t0, vSOA62);
    dst[2 * programCount + programIndex] = shuffle(t2, t4, vSOA63);
    dst[3 * programCount + programIndex] = shuffle(t1, t3, vSOA62);
    dst[4 * programCount + programIndex] = shuffle(t5, t1, vSOA64);
    dst[5 * programCount + programIndex] = shuffle(t3, t5, vSOA63);
}
#else
template<typename T>
unmasked inline void soa_to_aos6_ispc(const varying T &src0, const varying T &src1, const varying T &src2, const varying T &src3, const varying T &src4, const varying T &src5, uniform T dst[])
{
    const varying T t0 = shuffle(src0, src1, vSOA60);
    const varying T t1 = shuffle(src0, src1, vSOA61);
    const varying T t2 = shuffle(src2, src3, vSOA60);
    const varying T t3 = shuffle(src2, src3, vSOA61);
    const varying T t4 = shuffle(src4, src5, vSOA60);
    const varying T t5 = shuffle(src4, src5, vSOA61);

    dst[programIndex] = shuffle(shuffle(t0, t2, vSOA62), t4, vSOA63);
    dst[programIndex + programCount] = shuffle(shuffle(t0, t2, vSOA64), t4, vSOA65);
    dst[2 * programCount + programIndex] = shuffle(shuffle(t0, t2, vSOA66), t4, vSOA67);
    dst[3 * programCount + programIndex] = shuffle(shuffle(t1, t3, vSOA62), t5, vSOA63);
    dst[4 * programCount + programIndex] = shuffle(shuffle(t1, t3, vSOA64), t5, vSOA65);
    dst[5 * programCount + programIndex] = shuffle(shuffle(t1, t3, vSOA66), t5, vSOA67);
}
#endif

template<typename T>
unmasked inline void UniformStore(uniform T* uniform SrcPtr, uniform T* varying DstPtr)
{
    uniform T* uniform Dst[programCount];

    foreach(i = 0 ... programCount)
    {
        Dst[i] = (uniform T* varying)&DstPtr[0];
    }

    *Dst[0] = SrcPtr[0];
    *Dst[1] = SrcPtr[1];
    *Dst[2] = SrcPtr[2];
    *Dst[3] = SrcPtr[3];
#if TARGET_WIDTH == 8 || TARGET_WIDTH == 16
    *Dst[4] = SrcPtr[4];
    *Dst[5] = SrcPtr[5];
    *Dst[6] = SrcPtr[6];
    *Dst[7] = SrcPtr[7];
#endif
#if TARGET_WIDTH == 16
    *Dst[8] = SrcPtr[8];
    *Dst[9] = SrcPtr[9];
    *Dst[10] = SrcPtr[10];
    *Dst[11] = SrcPtr[11];
    *Dst[12] = SrcPtr[12];
    *Dst[13] = SrcPtr[13];
    *Dst[14] = SrcPtr[14];
    *Dst[15] = SrcPtr[15];
#endif
}

template<typename T>
unmasked inline void SoaToAos3Explicit(uniform T* uniform SrcPtr, uniform T* varying DstPtr)
{
    uniform T* uniform Dst[programCount];

    foreach(i = 0 ... programCount)
    {
        Dst[i] = (uniform T* varying)&DstPtr[0];
    }

#if TARGET_WIDTH == 4
    // X, X, X, X
    Dst[0][0] = SrcPtr[0];
    Dst[1][0] = SrcPtr[1];
    Dst[2][0] = SrcPtr[2];
    Dst[3][0] = SrcPtr[3];

    // Y, Y, Y, Y
    Dst[0][1] = SrcPtr[4];
    Dst[1][1] = SrcPtr[5];
    Dst[2][1] = SrcPtr[6];
    Dst[3][1] = SrcPtr[7];

    // Z, Z, Z, Z
    Dst[0][2] = SrcPtr[8];
    Dst[1][2] = SrcPtr[9];
    Dst[2][2] = SrcPtr[10];
    Dst[3][2] = SrcPtr[11];
#elif TARGET_WIDTH == 8
    // X, X, X, X, X, X, X, X
    Dst[0][0] = SrcPtr[0];
    Dst[1][0] = SrcPtr[1];
    Dst[2][0] = SrcPtr[2];
    Dst[3][0] = SrcPtr[3];
    Dst[4][0] = SrcPtr[4];
    Dst[5][0] = SrcPtr[5];
    Dst[6][0] = SrcPtr[6];
    Dst[7][0] = SrcPtr[7];

    // Y, Y, Y, Y, Y, Y, Y, Y
    Dst[0][1] = SrcPtr[8];
    Dst[1][1] = SrcPtr[9];
    Dst[2][1] = SrcPtr[10];
    Dst[3][1] = SrcPtr[11];
    Dst[4][1] = SrcPtr[12];
    Dst[5][1] = SrcPtr[13];
    Dst[6][1] = SrcPtr[14];
    Dst[7][1] = SrcPtr[15];

    // Z, Z, Z, Z, Z, Z, Z, Z
    Dst[0][2] = SrcPtr[16];
    Dst[1][2] = SrcPtr[17];
    Dst[2][2] = SrcPtr[18];
    Dst[3][2] = SrcPtr[19];
    Dst[4][2] = SrcPtr[20];
    Dst[5][2] = SrcPtr[21];
    Dst[6][2] = SrcPtr[22];
    Dst[7][2] = SrcPtr[23];
#elif TARGET_WIDTH == 16
    // X, X, X, X, X, X, X, X, X, X, X, X, X, X, X, X
    Dst[0][0] = SrcPtr[0];
    Dst[1][0] = SrcPtr[1];
    Dst[2][0] = SrcPtr[2];
    Dst[3][0] = SrcPtr[3];
    Dst[4][0] = SrcPtr[4];
    Dst[5][0] = SrcPtr[5];
    Dst[6][0] = SrcPtr[6];
    Dst[7][0] = SrcPtr[7];
    Dst[8][0] = SrcPtr[8];
    Dst[9][0] = SrcPtr[9];
    Dst[10][0] = SrcPtr[10];
    Dst[11][0] = SrcPtr[11];
    Dst[12][0] = SrcPtr[12];
    Dst[13][0] = SrcPtr[13];
    Dst[14][0] = SrcPtr[14];
    Dst[15][0] = SrcPtr[15];

    // Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y
    Dst[0][1] = SrcPtr[16];
    Dst[1][1] = SrcPtr[17];
    Dst[2][1] = SrcPtr[18];
    Dst[3][1] = SrcPtr[19];
    Dst[4][1] = SrcPtr[20];
    Dst[5][1] = SrcPtr[21];
    Dst[6][1] = SrcPtr[22];
    Dst[7][1] = SrcPtr[23];
    Dst[8][1] = SrcPtr[24];
    Dst[9][1] = SrcPtr[25];
    Dst[10][1] = SrcPtr[26];
    Dst[11][1] = SrcPtr[27];
    Dst[12][1] = SrcPtr[28];
    Dst[13][1] = SrcPtr[29];
    Dst[14][1] = SrcPtr[30];
    Dst[15][1] = SrcPtr[31];

    // Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z
    Dst[0][2] = SrcPtr[32];
    Dst[1][2] = SrcPtr[33];
    Dst[2][2] = SrcPtr[34];
    Dst[3][2] = SrcPtr[35];
    Dst[4][2] = SrcPtr[36];
    Dst[5][2] = SrcPtr[37];
    Dst[6][2] = SrcPtr[38];
    Dst[7][2] = SrcPtr[39];
    Dst[8][2] = SrcPtr[40];
    Dst[9][2] = SrcPtr[41];
    Dst[10][2] = SrcPtr[42];
    Dst[11][2] = SrcPtr[43];
    Dst[12][2] = SrcPtr[44];
    Dst[13][2] = SrcPtr[45];
    Dst[14][2] = SrcPtr[46];
    Dst[15][2] = SrcPtr[47];
#endif
}

template<typename T>
unmasked inline void SoaToAos4Explicit(uniform T* uniform SrcPtr, uniform T* varying DstPtr)
{
    uniform T* uniform Dst[programCount];

    foreach(i = 0 ... programCount)
    {
        Dst[i] = (uniform T* varying)&DstPtr[0];
    }

#if TARGET_WIDTH == 4
    // X, X, X, X
    Dst[0][0] = SrcPtr[0];
    Dst[1][0] = SrcPtr[1];
    Dst[2][0] = SrcPtr[2];
    Dst[3][0] = SrcPtr[3];

    // Y, Y, Y, Y
    Dst[0][1] = SrcPtr[4];
    Dst[1][1] = SrcPtr[5];
    Dst[2][1] = SrcPtr[6];
    Dst[3][1] = SrcPtr[7];

    // Z, Z, Z, Z
    Dst[0][2] = SrcPtr[8];
    Dst[1][2] = SrcPtr[9];
    Dst[2][2] = SrcPtr[10];
    Dst[3][2] = SrcPtr[11];

    // W, W, W, W
    Dst[0][3] = SrcPtr[12];
    Dst[1][3] = SrcPtr[13];
    Dst[2][3] = SrcPtr[14];
    Dst[3][3] = SrcPtr[15];
#elif TARGET_WIDTH == 8
    // X, X, X, X, X, X, X, X
    Dst[0][0] = SrcPtr[0];
    Dst[1][0] = SrcPtr[1];
    Dst[2][0] = SrcPtr[2];
    Dst[3][0] = SrcPtr[3];
    Dst[4][0] = SrcPtr[4];
    Dst[5][0] = SrcPtr[5];
    Dst[6][0] = SrcPtr[6];
    Dst[7][0] = SrcPtr[7];

    // Y, Y, Y, Y, Y, Y, Y, Y
    Dst[0][1] = SrcPtr[8];
    Dst[1][1] = SrcPtr[9];
    Dst[2][1] = SrcPtr[10];
    Dst[3][1] = SrcPtr[11];
    Dst[4][1] = SrcPtr[12];
    Dst[5][1] = SrcPtr[13];
    Dst[6][1] = SrcPtr[14];
    Dst[7][1] = SrcPtr[15];

    // Z, Z, Z, Z, Z, Z, Z, Z
    Dst[0][2] = SrcPtr[16];
    Dst[1][2] = SrcPtr[17];
    Dst[2][2] = SrcPtr[18];
    Dst[3][2] = SrcPtr[19];
    Dst[4][2] = SrcPtr[20];
    Dst[5][2] = SrcPtr[21];
    Dst[6][2] = SrcPtr[22];
    Dst[7][2] = SrcPtr[23];

    // W, W, W, W, W, W, W, W
    Dst[0][3] = SrcPtr[24];
    Dst[1][3] = SrcPtr[25];
    Dst[2][3] = SrcPtr[26];
    Dst[3][3] = SrcPtr[27];
    Dst[4][3] = SrcPtr[28];
    Dst[5][3] = SrcPtr[29];
    Dst[6][3] = SrcPtr[30];
    Dst[7][3] = SrcPtr[31];
#elif TARGET_WIDTH == 16
    // X, X, X, X, X, X, X, X, X, X, X, X, X, X, X, X
    Dst[0][0] = SrcPtr[0];
    Dst[1][0] = SrcPtr[1];
    Dst[2][0] = SrcPtr[2];
    Dst[3][0] = SrcPtr[3];
    Dst[4][0] = SrcPtr[4];
    Dst[5][0] = SrcPtr[5];
    Dst[6][0] = SrcPtr[6];
    Dst[7][0] = SrcPtr[7];
    Dst[8][0] = SrcPtr[8];
    Dst[9][0] = SrcPtr[9];
    Dst[10][0] = SrcPtr[10];
    Dst[11][0] = SrcPtr[11];
    Dst[12][0] = SrcPtr[12];
    Dst[13][0] = SrcPtr[13];
    Dst[14][0] = SrcPtr[14];
    Dst[15][0] = SrcPtr[15];

    // Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y, Y
    Dst[0][1] = SrcPtr[16];
    Dst[1][1] = SrcPtr[17];
    Dst[2][1] = SrcPtr[18];
    Dst[3][1] = SrcPtr[19];
    Dst[4][1] = SrcPtr[20];
    Dst[5][1] = SrcPtr[21];
    Dst[6][1] = SrcPtr[22];
    Dst[7][1] = SrcPtr[23];
    Dst[8][1] = SrcPtr[24];
    Dst[9][1] = SrcPtr[25];
    Dst[10][1] = SrcPtr[26];
    Dst[11][1] = SrcPtr[27];
    Dst[12][1] = SrcPtr[28];
    Dst[13][1] = SrcPtr[29];
    Dst[14][1] = SrcPtr[30];
    Dst[15][1] = SrcPtr[31];

    // Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z, Z
    Dst[0][2] = SrcPtr[32];
    Dst[1][2] = SrcPtr[33];
    Dst[2][2] = SrcPtr[34];
    Dst[3][2] = SrcPtr[35];
    Dst[4][2] = SrcPtr[36];
    Dst[5][2] = SrcPtr[37];
    Dst[6][2] = SrcPtr[38];
    Dst[7][2] = SrcPtr[39];
    Dst[8][2] = SrcPtr[40];
    Dst[9][2] = SrcPtr[41];
    Dst[10][2] = SrcPtr[42];
    Dst[11][2] = SrcPtr[43];
    Dst[12][2] = SrcPtr[44];
    Dst[13][2] = SrcPtr[45];
    Dst[14][2] = SrcPtr[46];
    Dst[15][2] = SrcPtr[47];

    // W, W, W, W, W, W, W, W, W, W, W, W, W, W, W, W
    Dst[0][3] = SrcPtr[48];
    Dst[1][3] = SrcPtr[49];
    Dst[2][3] = SrcPtr[50];
    Dst[3][3] = SrcPtr[51];
    Dst[4][3] = SrcPtr[52];
    Dst[5][3] = SrcPtr[53];
    Dst[6][3] = SrcPtr[54];
    Dst[7][3] = SrcPtr[55];
    Dst[8][3] = SrcPtr[56];
    Dst[9][3] = SrcPtr[57];
    Dst[10][3] = SrcPtr[58];
    Dst[11][3] = SrcPtr[59];
    Dst[12][3] = SrcPtr[60];
    Dst[13][3] = SrcPtr[61];
    Dst[14][3] = SrcPtr[62];
    Dst[15][3] = SrcPtr[63];
#endif
}

#endif
