// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/MathFwd.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"

/**
 * Support for Sobol quasi-random numbers
 */
class FSobol
{
public:
	// number of Sobol dimensions in DirectionNumbers and GrayNumbers tables (full Joe and Kuo table is 21201)
	static constexpr inline int32 MaxDimension = 15;

	// maximum number of bits in a 2D cell coordinate
	// allows cell grids from 1x1 to 2^MaxCell2DBits x 2^MaxCell2DBits
	static constexpr inline int32 MaxCell2DBits = 15;

	// maximum number of bits in a 3D cell coordinate
	// allows cell grids from 1x1x1 to 2^MaxCell3DBits x 2^MaxCell3DBits x 2^MaxCell3DBits
	static constexpr inline int32 MaxCell3DBits = 10;

private:
	// 24-bit Sobol direction numbers for 32-bit index
	static const int32 DirectionNumbers[MaxDimension + 1][32];

	// 24-bit Sobol direction numbers for Gray code order
	static const int32 GrayNumbers[MaxDimension + 1][32];

	// 24-bit Sobol direction numbers per cell
	static const int32 Cell2DDirectionNumbers[MaxCell2DBits + 1][32][2];

	// 24-bit Sobol direction numbers per cell in Gray-code order
	static const int32 Cell2DGrayNumbers[MaxCell2DBits + 1][32][2];

	// 24-bit Sobol direction numbers per cell
	static const int32 Cell3DDirectionNumbers[MaxCell3DBits + 1][32][3];

	// 24-bit Sobol direction numbers per cell in Gray-code order
	static const int32 Cell3DGrayNumbers[MaxCell3DBits + 1][32][3];

public:
	/**
	 * Evaluate Sobol number from one of the traditional Sobol dimensions at the given index
	 *
	 * @param Index - The index to evaluate.
	 * @param Dim - The Sobol dimension to use (0-15).
	 * @param Seed - A 24-bit random seed when reusing the same dimension
	 * @return The Sobol result at the given Index
	 */
	[[nodiscard]] CORE_API static float Evaluate(int32 Index, int32 Dim, int32 Seed = 0);

	/**
	* Evaluate next Sobol number from one of the traditional Sobol dimensions
	*
	* @param Index - The index for the Sobol number to generate.
	* @param Dim - The Sobol dimension to use (0-15)
	* @param Value - The value for the Sobol number at Index-1.
	* @return The Sobol result at the given Index
	*/
	[[nodiscard]] CORE_API static float Next(int32 Index, int32 Dim, float Value);


	/**
	* Evaluate Sobol number from within a 2D cell at given index
	*
	* @param Index - The index to evaluate.
	* @param Cell - Integer cell coordinates.
	* @param CellBits - Number of bits in cell coordinates.
	* @param Seed - A 24-bit per component 2D seed for shuffling values
	* @return The 2D Sobol result in the range 0-1 given Index
	*/
	[[nodiscard]] CORE_API static FVector2D Evaluate(int32 Index, int32 CellBits, FIntPoint Cell, FIntPoint Seed);

	/**
	* Evaluate next Sobol number from within a 2D cell
	*
	* @param Index - The index for the Sobol number to generate.
	* @param CellBits - Number of bits in cell coordinates.
	* @param Value - The value for the Sobol number at Index-1.
	* @return The 2D Sobol result in the range 0-1 given Index
	*/
	[[nodiscard]] CORE_API static FVector2D Next(int32 Index, int32 CellBits, FVector2D Value);


	/**
	* Evaluate Sobol number from within a 3D cell at given index
	*
	* @param Index - The index to evaluate.
	* @param Cell - Integer cell coordinates.
	* @param CellBits - Number of bits in cell coordinates.
	* @param Seed - A seed for shuffling values (0-1)
	* @return The Sobol result given Index
	*/
	[[nodiscard]] CORE_API static FVector Evaluate(int32 Index, int32 CellBits, FIntVector Cell, FIntVector Seed);

	/**
	* Evaluate next Sobol number from within a 3D cell
	*
	* @param Index - The index for the Sobol number to generate.
	* @param CellBits - Number of bits in cell coordinates.
	* @param Value - The value for the Sobol number at Index-1.
	* @return The Sobol result at given Index
	*/
	[[nodiscard]] CORE_API static FVector Next(int32 Index, int32 CellBits, FVector Value);

	/** Return value for GSystemTextures.SobolSampling */
	[[nodiscard]] CORE_API static uint16 ComputeGPUSpatialSeed(int32 x, int32 y, int32 Index);
};