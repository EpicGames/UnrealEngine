// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/MathFwd.h" // IWYU pragma: export
#include "Math/Matrix.h"
#include "Math/Transform.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Misc/AssertionMacros.h"
#include "Misc/LargeWorldCoordinates.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Templates/IsUECoreType.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/ObjectVersion.h"

/**
 * Implements a basic sphere.
 */
namespace UE {
namespace Math {
template <typename T> struct TMatrix;

template<typename T>
struct TSphere
{
public:
	using FReal = T;

	/** The sphere's center point. */
	TVector<T> Center;

	/** The sphere's radius. */
	T W;


	/** Default constructor (no initialization). */
	[[nodiscard]] TSphere() = default;

	/**
	 * Creates and initializes a new sphere.
	 *
	 * @param int32 Passing int32 sets up zeroed sphere.
	 */
	[[nodiscard]] TSphere(int32)
		: Center(0.0f, 0.0f, 0.0f)
		, W(0)
	{ }

	/**
	 * Creates and initializes a new sphere with the specified parameters.
	 *
	 * @param InV Center of sphere.
	 * @param InW Radius of sphere.
	 */
	[[nodiscard]] TSphere(TVector<T> InV, T InW)
		: Center(InV)
		, W(InW)
	{ }

	/**
	 * Constructor.
	 *
	 * @param EForceInit Force Init Enum.
	 */
	[[nodiscard]] explicit inline TSphere(EForceInit)
		: Center(ForceInit)
		, W(0.0f)
	{ }

	/**
	 * Constructor.
	 *
	 * @param Points Pointer to list of points this sphere must contain.
	 * @param Count How many points are in the list.
	 */
	[[nodiscard]] CORE_API TSphere(const TVector<T>* Points, int32 Count);


	/**
	 * Constructor.
	 *
	 * @param Spheres Pointer to list of spheres this sphere must contain.
	 * @param Count How many points are in the list.
	 */
	[[nodiscard]] CORE_API TSphere(const TSphere<T>* Spheres, int32 Count);

	// Conversion from other variant type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	[[nodiscard]] explicit TSphere(const TSphere<FArg>& From)
		: TSphere<T>(TVector<T>(From.Center), T(From.W))
	{
	}

	/**
	 * Check whether two spheres are the same within specified tolerance.
	 *
	 * @param Sphere The other sphere.
	 * @param Tolerance Error Tolerance.
	 * @return true if spheres are equal within specified tolerance, otherwise false.
	 */
	[[nodiscard]] bool Equals(const TSphere<T>& Sphere, T Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return Center.Equals(Sphere.Center, Tolerance) && FMath::Abs(W - Sphere.W) <= Tolerance;
	}

	/**
	* Compares two spheres for equality.
	*
	* @param Other The other sphere to compare with.
	* @return true if the spheres are equal, false otherwise.
	*/
	[[nodiscard]] bool operator==(const TSphere<T>& Other) const
	{
		return Center == Other.Center && W == Other.W;
	}

	/**
	 * Compares two spheres for inequality.
	 *
	 * @param Other The other sphere to compare with.
	 * @return true if the spheres are not equal, false otherwise.
	 */
	[[nodiscard]] bool operator!=(const TSphere<T>& Other) const
	{
		return !(*this == Other);
	}

	/**
	 * Gets the result of addition to this bounding volume.
	 *
	 * @param Other The other volume to add to this.
	 * @return A new bounding volume.
	 */
	[[nodiscard]] TSphere<T> operator+(const TSphere<T>& Other) const
	{
		return TSphere<T>(*this) += Other;
	}

	/**
	 * Adds to this sphere to include a new bounding volume.
	 *
	 * @param Other the bounding volume to increase the bounding volume to.
	 * @return Reference to this bounding volume after resizing to include the other bounding volume.
	 */
	TSphere<T>& operator+=(const TSphere<T>& Other);

	/**
	 * Check whether sphere is inside of another.
	 *
	 * @param Other The other sphere.
	 * @param Tolerance Error Tolerance.
	 * @return true if sphere is inside another, otherwise false.
	 */
	[[nodiscard]] bool IsInside(const TSphere<T>& Other, T Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		if (W > Other.W + Tolerance)
		{
			return false;
		}

		return (Center - Other.Center).SizeSquared() <= FMath::Square(Other.W + Tolerance - W);
	}

	/**
	* Checks whether the given location is inside this sphere.
	*
	* @param In The location to test for inside the bounding volume.
	* @return true if location is inside this volume.
	*/
	[[nodiscard]] bool IsInside(const FVector& In, T Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return (Center - In).SizeSquared() <= FMath::Square(W + Tolerance);
	}

	/**
	 * Test whether this sphere intersects another.
	 * 
	 * @param  Other The other sphere.
	 * @param  Tolerance Error tolerance.
	 * @return true if spheres intersect, false otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool Intersects(const TSphere<T>& Other, T Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return (Center - Other.Center).SizeSquared() <= FMath::Square(FMath::Max(0.f, Other.W + W + Tolerance));
	}

	/**
	 * Get result of Transforming sphere by Matrix.
	 *
	 * @param M Matrix to transform by.
	 * @return Result of transformation.
	 */
	[[nodiscard]] TSphere<T> TransformBy(const TMatrix<T>& M) const;

	/**
	 * Get result of Transforming sphere with Transform.
	 *
	 * @param M Transform information.
	 * @return Result of transformation.
	 */
	[[nodiscard]] TSphere<T> TransformBy(const FTransform& M) const;

	/**
	 * Get volume of the current sphere
	 *
	 * @return Volume (in Unreal units).
	 */
	[[nodiscard]] T GetVolume() const
	{
		return (4.f / 3.f) * UE_PI * (W * W * W);
	}

	/**
	* Get a textual representation of the sphere.
	*
	* @return Text describing the sphere.
	*/
	[[nodiscard]] FString ToString() const
	{
		return FString::Printf(TEXT("Center=(%s), Radius=(%s)"), *Center.ToString(), *LexToSanitizedString(W));
	}

	// Note: TSphere is usually written via binary serialization. This function exists for SerializeFromMismatchedTag conversion usage. 
	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		if constexpr (std::is_same_v<T, float>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Sphere, Sphere3f, Sphere3d);
		}
		else if constexpr (std::is_same_v<T, double>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Sphere, Sphere3d, Sphere3f);
		}
		else
		{
			static_assert(sizeof(T) == 0, "Unimplemented");
			return false;
		}
	}

};

/**
 * Serializes the given sphere from or into the specified archive.
 *
 * @param Ar The archive to serialize from or into.
 * @param Sphere The sphere to serialize.
 * @return The archive.
 */

inline FArchive& operator<<(FArchive& Ar, TSphere<float>& Sphere)
{
	Ar << Sphere.Center << Sphere.W;
	return Ar;
}

/**
 * Serializes the given sphere from or into the specified archive.
 *
 * @param Ar The archive to serialize from or into.
 * @param Sphere The sphere to serialize.
 * @return The archive.
 */

inline FArchive& operator<<(FArchive& Ar, TSphere<double>& Sphere)
{
	Ar << Sphere.Center;

	if (Ar.UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
	{
		Ar << Sphere.W;
	}
	else
	{
		checkf(Ar.IsLoading(), TEXT("float -> double conversion applied outside of load!"));
		// Stored as floats, so serialize float and copy.
		float SW;
		Ar << SW;
		Sphere.W = (double)SW;
	}

	return Ar;
}

template<typename T>
TSphere<T> TSphere<T>::TransformBy(const TMatrix<T>& M) const
{
	TSphere<T>	Result;

	TVector4<T> TransformedCenter = M.TransformPosition(this->Center);
	Result.Center = TVector<T>(TransformedCenter.X, TransformedCenter.Y, TransformedCenter.Z);

	const TVector<T> XAxis(M.M[0][0], M.M[0][1], M.M[0][2]);
	const TVector<T> YAxis(M.M[1][0], M.M[1][1], M.M[1][2]);
	const TVector<T> ZAxis(M.M[2][0], M.M[2][1], M.M[2][2]);

	Result.W = FMath::Sqrt(FMath::Max(XAxis | XAxis, FMath::Max(YAxis | YAxis, ZAxis | ZAxis))) * W;

	return Result;
}


template<typename T>
TSphere<T> TSphere<T>::TransformBy(const FTransform& M) const
{
	TSphere<T>	Result;

	Result.Center = M.TransformPosition(this->Center);
	Result.W = M.GetMaximumAxisScale() * W;

	return Result;
}

template<typename T>
TSphere<T>& TSphere<T>::operator+=(const TSphere<T>& Other)
{
	if (W == 0.f)
	{
		*this = Other;
		return *this;
	}

	TVector<T> ToOther = Other.Center - Center;
	T DistSqr = ToOther.SizeSquared();

	if (FMath::Square(W - Other.W) + UE_KINDA_SMALL_NUMBER >= DistSqr)
	{
		// Pick the smaller
		if (W < Other.W)
		{
			*this = Other;
		}
	}
	else
	{
		T Dist = FMath::Sqrt(DistSqr);

		TSphere<T> NewSphere;
		NewSphere.W = (Dist + Other.W + W) * 0.5f;
		NewSphere.Center = Center;

		if (Dist > UE_SMALL_NUMBER)
		{
			NewSphere.Center += ToOther * ((NewSphere.W - W) / Dist);
		}

		// make sure both are inside afterwards
		checkSlow(Other.IsInside(NewSphere, 1.f));
		checkSlow(IsInside(NewSphere, 1.f));

		*this = NewSphere;
	}

	return *this;
}

// Forward declarations for complex constructors.
template<> CORE_API TSphere<float>::TSphere(const TVector<float>* Points, int32 Count);
template<> CORE_API TSphere<double>::TSphere(const TVector<double>* Points, int32 Count);
template<> CORE_API TSphere<float>::TSphere(const TSphere<float>* Spheres, int32 Count);
template<> CORE_API TSphere<double>::TSphere(const TSphere<double>* Spheres, int32 Count);

} // namespace Math
} // namespace UE


UE_DECLARE_LWC_TYPE(Sphere, 3);

template<> struct TCanBulkSerialize<FSphere3f> { enum { Value = true }; };
template<> struct TIsPODType<FSphere3f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FSphere3f> { enum { Value = true }; };

template<> struct TCanBulkSerialize<FSphere3d> { enum { Value = true }; };
template<> struct TIsPODType<FSphere3d> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FSphere3d> { enum { Value = true }; };

/* FMath inline functions
 *****************************************************************************/

/**
* Computes minimal bounding sphere encompassing given cone
*/
template<typename FReal>
inline UE::Math::TSphere<FReal> FMath::ComputeBoundingSphereForCone(UE::Math::TVector<FReal> const& ConeOrigin, UE::Math::TVector<FReal> const& ConeDirection, FReal ConeRadius, FReal CosConeAngle, FReal SinConeAngle)
{
	// Based on: https://bartwronski.com/2017/04/13/cull-that-cone/
	const FReal COS_PI_OVER_4 = 0.70710678118f; // Cos(Pi/4);		// LWC_TODO: precision improvement possible here
	if (CosConeAngle < COS_PI_OVER_4)
	{
		return UE::Math::TSphere<FReal>(ConeOrigin + ConeDirection * ConeRadius * CosConeAngle, ConeRadius * SinConeAngle);
	}
	else
	{
		const FReal BoundingRadius = ConeRadius / (2.0f * CosConeAngle);
		return UE::Math::TSphere<FReal>(ConeOrigin + ConeDirection * BoundingRadius, BoundingRadius);
	}
}
