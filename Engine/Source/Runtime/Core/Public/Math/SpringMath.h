// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Math/Vector.h"
#include "Math/Quat.h"

// Collection of useful spring methods which can be used for damping, simulating characters etc.
// 
// Reference https://theorangeduck.com/page/spring-roll-call
//		     https://theorangeduck.com/page/scalar-velocity
//			 https://theorangeduck.com/page/dead-blending
//			 https://theorangeduck.com/page/propagating-velocities-through-animation-systems
//			 https://theorangeduck.com/page/fitting-code-driven-displacement
//			 https://theorangeduck.com/page/fitting-code-driven-displacement-revisited

struct SpringMath
{
private:
	static constexpr float SmoothingTimeToDamping(float SmoothingTime)
	{
		return 4.0f / FMath::Max(SmoothingTime, UE_SMALL_NUMBER);
	}

	static constexpr float DampingToSmoothingTime(float Damping)
	{
		return 4.0f / FMath::Max(Damping, UE_SMALL_NUMBER);
	}

	/** Divides one scale by another, clamping the Rhs to be greater than some minimum */
	static inline FVector ScaleDivMax(FVector Lhs, FVector Rhs)
	{
		return FVector(
			Lhs.X / FMath::Max(Rhs.X, UE_SMALL_NUMBER),
			Lhs.Y / FMath::Max(Rhs.Y, UE_SMALL_NUMBER),
			Lhs.Z / FMath::Max(Rhs.Z, UE_SMALL_NUMBER));
	}

	/** Takes the log of a scale, clamping the scale to ensure it is greater than zero */
	static inline FVector ScaleLog(FVector V)
	{
		return FVector(
			FMath::Loge(FMath::Max(V.X, UE_SMALL_NUMBER)),
			FMath::Loge(FMath::Max(V.Y, UE_SMALL_NUMBER)),
			FMath::Loge(FMath::Max(V.Z, UE_SMALL_NUMBER)));
	}

	/** Computes the exponent of a scale, ensuring the value is not too large to prevent it exploding */
	static inline FVector ScaleExp(FVector V, double Max = 10.0)
	{
		return FVector(
			FMath::Exp(FMath::Min(V.X, Max)),
			FMath::Exp(FMath::Min(V.Y, Max)),
			FMath::Exp(FMath::Min(V.Z, Max)));
	}

	/** Performs an Eerp on scales (interpolation in the multiplicative space) */
	static inline FVector ScaleEerp(FVector A, FVector B, double Alpha)
	{
		return ScaleExp(FMath::Lerp(ScaleLog(A), ScaleLog(B), Alpha));
	}

	/** Gets the version of angle A which is closest to reference angle B */
	static inline float AngleGetShortestArcWith(float A, float B)
	{
		return FMath::FindDeltaAngleRadians(B, A) + B;
	}

	/** Computes the lower (-1) branch of the LambertW function, which is used for solving equations of the form x*e^x - input will be clamped to range [-1/e, 0], output will be in range [-1, -inf] */
	static inline float LambertWLower(const float X)
	{
		// Clamp to valid domain
		const float XValid = FMath::Clamp(X, -1.0f / UE_EULERS_NUMBER + UE_SMALL_NUMBER, -UE_SMALL_NUMBER);

		const float Redx = 0.625529587f * 0.588108778f + XValid;
		
		// Polynomial approximation for when close to -(exp(-1))
		if (Redx <= 0.09765625f)
		{
			const float R = FMath::Sqrt(Redx);
			float W = -3.30250000e+2f;
			W = W * R + 3.53563141e+2f;
			W = W * R - 1.91617889e+2f;
			W = W * R + 4.94172478e+1f;
			W = W * R - 1.23464909e+1f;
			W = W * R - 1.38704872e+0f;
			W = W * R - 1.99431837e+0f;
			W = W * R - 1.81044364e+0f;
			W = W * R - 2.33166337e+0f;
			W = W * R - 1.00000000e+0f;
			return W;
		}
		// From "New approximations to the principal real - valued branch of the Lambert W function"
		else
		{
			const float C0 = +1.68090820e-1f;
			const float C1 = -2.96497345e-3f;
			const float C2 = -2.87322998e-2f;
			const float C3 = +7.07275391e-1f;

			const float S = FMath::Log2(-XValid) * -UE_LN2 - 1.0f;
			float T = FMath::Sqrt(S);
			float W = -1.0f - S - (1.0f / (FMath::Exp2(C2 * T) * (C1 * T) + C3 / T + C0));

			if (XValid > -0x1.0p-116f)
			{
				const float E = FMath::Exp2(W / UE_LN2 + 32.0f);
				W = -(W * E - 0x1.0p32f * XValid) / (W * E + E) + W;
			}
			else
			{
				T = W / (1.0f + W);
				W = T * FMath::Loge(XValid / W) + T;
			}

			return W;
		}
	}

public:
	/** Convert a smoothing time to a half life
	 * 
	 * @param SmoothingTime The smoothing time of the spring in seconds. It is the time by which the output lags the input when critically damped, when the input is changing at a constant rate. 
	 * @return The half life of the spring. How long it takes the value to get halfway towards the target. 
	 */
	static constexpr float SmoothingTimeToHalfLife(float SmoothingTime)
	{
		return SmoothingTime * UE_LN2;
	}

	/** Convert a halflife to a smoothing time
	 * 
	 * @param HalfLife The half life of the spring. How long it takes the value to get halfway towards the target.
	 * @return The smoothing time of the spring in seconds. It is the time by which the output lags the input when critically damped, when the input is changing at a constant rate.
	 */
	static constexpr float HalfLifeToSmoothingTime(float HalfLife)
	{
		return HalfLife / UE_LN2;
	}

	/** Convert from smoothing time to spring strength.
	  * 
	 * @param SmoothingTime The smoothing time of the spring in seconds. It is the time by which the output lags the input when critically damped, when the input is changing at a constant rate.
	 * @return The spring strength. This corresponds to the undamped frequency of the spring in hz.
	 */
	static constexpr float SmoothingTimeToStrength(float SmoothingTime)
	{
		return 2.0f / FMath::Max(SmoothingTime, UE_SMALL_NUMBER);
	}

	/** Convert from spring strength to smoothing time.
	 * 
	 * @param Strength The spring strength. This corresponds to the undamped frequency of the spring in hz.
	 * @return The smoothing time of the spring in seconds. It is the time by which the output lags the input when critically damped, when the input is changing at a constant rate.
	 */
	static constexpr float StrengthToSmoothingTime(float Strength)
	{
		return 2.0f / FMath::Max(Strength, UE_SMALL_NUMBER);
	}

	/** Simplified version of FMath::CriticallyDampedSmoothing where v_goal is assumed to be 0. This interpolates the value InOutX towards TargetX
	 * with the motion of a critically damped spring. The velocity of InOutX is stored in InOutV.
	 * 
	 * @tparam T The type to be damped 
	 * @param InOutX The value to be damped
	 * @param InOutV The velocity of the value to be damped
	 * @param TargetX The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutX to TargetX
	 * @param DeltaTime Timestep in seconds
	 */
	template <typename T>
	static void CriticalSpringDamper(
		T& InOutX,
		T& InOutV,
		T TargetX,
		float SmoothingTime,
		float DeltaTime)
	{
		if (SmoothingTime < UE_SMALL_NUMBER)
		{
			InOutX = TargetX;
			InOutV = T(0);
			return;
		}

		float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;
		T J0 = InOutX - TargetX;
		T J1 = InOutV + J0 * Y;
		float EyDt = FMath::InvExpApprox(Y * DeltaTime);

		InOutX = EyDt * (J0 + J1 * DeltaTime) + TargetX;
		InOutV = EyDt * (InOutV - J1 * Y * DeltaTime);
	}

	/** Specialized angle version of CriticalSpringDamper that handles angle wrapping.
	 *
	 * @param InOutAngleRadians The value to be damped
	 * @param InOutAngularVelocityRadians The velocity of the value to be damped
	 * @param TargetAngleRadians The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutAngleRadians to TargetAngleRadians
	 * @param DeltaTime Timestep in seconds
	 */
	static void CriticalSpringDamperAngle(
		float& InOutAngleRadians,
		float& InOutAngularVelocityRadians,
		float TargetAngleRadians,
		float SmoothingTime,
		float DeltaTime)
	{
		if (SmoothingTime < UE_SMALL_NUMBER)
		{
			InOutAngleRadians = TargetAngleRadians;
			InOutAngularVelocityRadians = 0.0f;
			return;
		}

		float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;

		float J0 = FMath::FindDeltaAngleRadians(TargetAngleRadians, InOutAngleRadians);
		float J1 = InOutAngularVelocityRadians + J0 * Y;
		float EyDt = FMath::InvExpApprox(Y * DeltaTime);

		InOutAngleRadians = EyDt * (J0 + J1 * DeltaTime) + TargetAngleRadians;
		InOutAngularVelocityRadians = EyDt * (InOutAngularVelocityRadians - J1 * Y * DeltaTime);
	}

	/** Specialized quaternion version of CriticalSpringDamper, uses FVector for angular velocity
	 *
	 * @param InOutRotation The value to be damped
	 * @param InOutAngularVelocityRadians The angular velocity of the rotation in radians
	 * @param TargetRotation The target rotation to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutRotation to TargetRotation
	 * @param DeltaTime Timestep in seconds
	 */
	static void CriticalSpringDamperQuat(
		FQuat& InOutRotation,
		FVector& InOutAngularVelocityRadians,
		const FQuat& TargetRotation,
		float SmoothingTime,
		float DeltaTime)
	{
		if (SmoothingTime < UE_SMALL_NUMBER)
		{
			InOutRotation = TargetRotation;
			InOutAngularVelocityRadians = FVector::ZeroVector;
			return;
		}

		float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;

		FVector J0 = (InOutRotation * TargetRotation.Inverse()).GetShortestArcWith(FQuat::Identity).ToRotationVector();
		FVector J1 = InOutAngularVelocityRadians + J0 * Y;

		float EyDt = FMath::InvExpApprox(Y * DeltaTime);
		
		InOutRotation = FQuat::MakeFromRotationVector(EyDt * (J0 + J1 * DeltaTime)) * TargetRotation;
		InOutAngularVelocityRadians = EyDt * (InOutAngularVelocityRadians - J1 * Y * DeltaTime);
	}

	/** Specialized scale version of CriticalSpringDamper, uses FVector for scalar velocity
	 *
	 * @param InOutScale The value to be damped
	 * @param InOutScalarVelocity The scales velocity of the scale
	 * @param TargetScale The target scale to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutScale to TargetScale
	 * @param DeltaTime Timestep in seconds
	 */
	static void CriticalSpringDamperScale(
		FVector& InOutScale,
		FVector& InOutScalarVelocity,
		const FVector& TargetScale,
		float SmoothingTime,
		float DeltaTime)
	{
		if (SmoothingTime < UE_SMALL_NUMBER)
		{
			InOutScale = TargetScale;
			InOutScalarVelocity = FVector::ZeroVector;
			return;
		}

		float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;

		FVector J0 = ScaleLog(ScaleDivMax(InOutScale, TargetScale));
		FVector J1 = InOutScalarVelocity + J0 * Y;

		float EyDt = FMath::InvExpApprox(Y * DeltaTime);

		InOutScale = ScaleExp(EyDt * (J0 + J1 * DeltaTime))* TargetScale;
		InOutScalarVelocity = EyDt * (InOutScalarVelocity - J1 * Y * DeltaTime);
	}

	/** Simplified version of CriticalSpringDamper where the target is assumed to be 0. This decays the value InOutX towards zero
	 * with the motion of a critically damped spring. The velocity of InOutX is stored in InOutV.
	 *
	 * @tparam T The type to be decayed
	 * @param InOutX The value to be decayed
	 * @param InOutV The velocity of the value to be decayed
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutX to zero
	 * @param DeltaTime Timestep in seconds
	 */
	template <typename T>
	static void CriticalDecay(
		T& InOutX,
		T& InOutV,
		float SmoothingTime,
		float DeltaTime)
	{
		if (SmoothingTime < UE_SMALL_NUMBER)
		{
			InOutX = T(0);
			InOutV = T(0);
			return;
		}

		float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;
		T J0 = InOutX;
		T J1 = InOutV + J0 * Y;
		float EyDt = FMath::InvExpApprox(Y * DeltaTime);

		InOutX = EyDt * (J0 + J1 * DeltaTime);
		InOutV = EyDt * (InOutV - J1 * Y * DeltaTime);
	}

	/** Specialized angle version of CriticalDecay that handles angle wrapping.
	 *
	 * @param InOutAngleRadians The value to be decayed
	 * @param InOutAngularVelocityRadians The velocity of the value to be decayed
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutAngleRadians to TargetAngleRadians
	 * @param DeltaTime Timestep in seconds
	 */
	static void CriticalDecayAngle(
		float& InOutAngleRadians,
		float& InOutAngularVelocityRadians,
		float SmoothingTime,
		float DeltaTime)
	{
		if (SmoothingTime < UE_SMALL_NUMBER)
		{
			InOutAngleRadians = 0.0f;
			InOutAngularVelocityRadians = 0.0f;
			return;
		}

		float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;

		float J0 = InOutAngleRadians;
		float J1 = InOutAngularVelocityRadians + J0 * Y;
		float EyDt = FMath::InvExpApprox(Y * DeltaTime);

		InOutAngleRadians = EyDt * (J0 + J1 * DeltaTime);
		InOutAngularVelocityRadians = EyDt * (InOutAngularVelocityRadians - J1 * Y * DeltaTime);
	}

	/** Specialized quaternion version of CriticalDecay, uses FVector for angular velocity
	 *
	 * @param InOutRotation The value to be decayed
	 * @param InOutAngularVelocityRadians The angular velocity of the rotation in radians
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutRotation to TargetRotation
	 * @param DeltaTime Timestep in seconds
	 */
	static void CriticalDecayQuat(
		FQuat& InOutRotation,
		FVector& InOutAngularVelocityRadians,
		float SmoothingTime,
		float DeltaTime)
	{
		if (SmoothingTime < UE_SMALL_NUMBER)
		{
			InOutRotation = FQuat::Identity;
			InOutAngularVelocityRadians = FVector::ZeroVector;
			return;
		}

		float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;

		FVector J0 = InOutRotation.GetShortestArcWith(FQuat::Identity).ToRotationVector();
		FVector J1 = InOutAngularVelocityRadians + J0 * Y;

		float EyDt = FMath::InvExpApprox(Y * DeltaTime);

		InOutRotation = FQuat::MakeFromRotationVector(EyDt * (J0 + J1 * DeltaTime));
		InOutAngularVelocityRadians = EyDt * (InOutAngularVelocityRadians - J1 * Y * DeltaTime);
	}

	/** Specialized scale version of CriticalDecay
	 *
	 * @param InOutScale The value to be decayed
	 * @param InOutScalarVelocity The scalar velocity of the scale
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutRotation to TargetRotation
	 * @param DeltaTime Timestep in seconds
	 */
	static void CriticalDecayScale(
		FVector& InOutScale,
		FVector& InOutScalarVelocity,
		float SmoothingTime,
		float DeltaTime)
	{
		if (SmoothingTime < UE_SMALL_NUMBER)
		{
			InOutScale = FVector::OneVector;
			InOutScalarVelocity = FVector::ZeroVector;
			return;
		}

		float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;

		FVector J0 = ScaleLog(InOutScale);
		FVector J1 = InOutScalarVelocity + J0 * Y;

		float EyDt = FMath::InvExpApprox(Y * DeltaTime);

		InOutScale = ScaleExp(EyDt * (J0 + J1 * DeltaTime));
		InOutScalarVelocity = EyDt * (InOutScalarVelocity - J1 * Y * DeltaTime);
	}

	/** A double spring damper that interpolates the value InOutX towards TargetX via some intermediate state - producing a more S-shaped curve. 
	 * The velocity of InOutX is stored in InOutV.
	 *
	 * @tparam T The type to be damped
	 * @param InOutX The value to be damped
	 * @param InOutV The velocity of the value to be damped
	 * @param InOutXi The value of the intermediate state
	 * @param InOutVi The velocity of the intermediate state
	 * @param TargetX The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutX to TargetX
	 * @param DeltaTime Timestep in seconds
	 */
	template <typename T>
	static void CriticalDoubleSpringDamper(
		T& InOutX,
		T& InOutV,
		T& InOutXi,
		T& InOutVi,
		T TargetX,
		float SmoothingTime,
		float DeltaTime)
	{
		CriticalSpringDamper(InOutXi, InOutVi, TargetX, SmoothingTime / 2.0f, DeltaTime);
		CriticalSpringDamper(InOutX, InOutV, InOutXi, SmoothingTime / 2.0f, DeltaTime);
	}

	/** Specialized angle version of CriticalDoubleSpringDamper that handles angle wrapping.
	 *
	 * @param InOutAngleRadians The value to be damped
	 * @param InOutAngularVelocityRadians The velocity of the value to be damped
	 * @param InOutIntermediateAngleRadians The value of the intermediate state
	 * @param InOutIntermediateAngularVelocityRadians The velocity of the intermediate state
	 * @param TargetAngleRadians The goal to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutAngleRadians to TargetAngleRadians
	 * @param DeltaTime Timestep in seconds
	 */
	static void CriticalDoubleSpringDamperAngle(
		float& InOutAngleRadians,
		float& InOutAngularVelocityRadians,
		float& InOutIntermediateAngleRadians,
		float& InOutIntermediateAngularVelocityRadians,
		float TargetAngleRadians,
		float SmoothingTime,
		float DeltaTime)
	{
		CriticalSpringDamperAngle(InOutIntermediateAngleRadians, InOutIntermediateAngularVelocityRadians, TargetAngleRadians, SmoothingTime / 2.0f, DeltaTime);
		CriticalSpringDamperAngle(InOutAngleRadians, InOutAngularVelocityRadians, InOutIntermediateAngleRadians, SmoothingTime / 2.0f, DeltaTime);
	}

	/** Specialized quaternion version of CriticalDoubleSpringDamper, uses FVector for angular velocity
	 *
	 * @param InOutRotation The value to be damped
	 * @param InOutAngularVelocityRadians The angular velocity of the rotation in radians
	 * @param InOutIntermediateRotation The value of the intermediate state
	 * @param InOutIntermediateAngularVelocityRadians The velocity of the intermediate state
	 * @param TargetRotation The target rotation to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutRotation to TargetRotation
	 * @param DeltaTime Timestep in seconds
	 */
	static void CriticalDoubleSpringDamperQuat(
		FQuat& InOutRotation,
		FVector& InOutAngularVelocityRadians,
		FQuat& InOutIntermediateRotation,
		FVector& InOutIntermediateAngularVelocityRadians,
		const FQuat& TargetRotation,
		float SmoothingTime,
		float DeltaTime)
	{
		CriticalSpringDamperQuat(InOutIntermediateRotation, InOutIntermediateAngularVelocityRadians, TargetRotation, SmoothingTime / 2.0f, DeltaTime);
		CriticalSpringDamperQuat(InOutRotation, InOutAngularVelocityRadians, InOutIntermediateRotation, SmoothingTime / 2.0f, DeltaTime);
	}

	/** Specialized scale version of CriticalDoubleSpringDamper
	 *
	 * @param InOutScale The value to be damped
	 * @param InOutScalarVelocity The scalar velocity of the scale
	 * @param InOutIntermediateScale The value of the intermediate state
	 * @param InOutIntermediateScalarVelocity The velocity of the intermediate state
	 * @param TargetScale The target scale to damp towards
	 * @param SmoothingTime The smoothing time to use for the spring. Longer times lead to more damped behaviour. A time of 0 will snap InOutScale to TargetScale
	 * @param DeltaTime Timestep in seconds
	 */
	static void CriticalDoubleSpringDamperScale(
		FVector& InOutScale,
		FVector& InOutScalarVelocity,
		FVector& InOutIntermediateScale,
		FVector& InOutIntermediateScalarVelocity,
		const FVector& TargetScale,
		float SmoothingTime,
		float DeltaTime)
	{
		CriticalSpringDamperScale(InOutIntermediateScale, InOutIntermediateScalarVelocity, TargetScale, SmoothingTime / 2.0f, DeltaTime);
		CriticalSpringDamperScale(InOutScale, InOutScalarVelocity, InOutIntermediateScale, SmoothingTime / 2.0f, DeltaTime);
	}

	/** A velocity spring will damp towards a target that follows a fixed linear target velocity, allowing control of the interpolation speed
	 * while still giving a smoothed behavior. A SmoothingTime of 0 will give a linear interpolation between X and TargetX
	 * 
	 * @param InOutX The value to be damped
	 * @param InOutV The velocity of the value to be damped
	 * @param InOutXi The intermediate target of the value to be damped
	 * @param TargetX The target value of X to damp towards
	 * @param MaxSpeed The desired speed to achieve while damping towards X
	 * @param SmoothingTime The smoothing time to use while damping towards X. Higher values will give more smoothed behaviour. A value of 0 will give a linear interpolation of X to Target
	 * @param DeltaTime The timestep in seconds
	 */
	template <typename TFloat>
	static void VelocitySpringDamperF(
		TFloat& InOutX,
		TFloat& InOutV,
		TFloat& InOutXi,
		TFloat TargetX,
		TFloat MaxSpeed,
		float SmoothingTime,
		float DeltaTime)
	{
		static_assert(std::is_floating_point_v<TFloat>, "TFloat must be floating point");

		MaxSpeed = FMath::Max(MaxSpeed, 0.0f); // MaxSpeed can't be negative

		TFloat XDiff = ((TargetX - InOutXi) > 0.0f
			? 1.0f
			: -1.0f) * MaxSpeed;

		float TGoalFuture = SmoothingTime;
		TFloat XGoalFuture = FMath::Abs(TargetX - InOutXi) > TGoalFuture * MaxSpeed
			? InOutXi + XDiff * TGoalFuture
			: TargetX;

		CriticalSpringDamper(InOutX, InOutV, XGoalFuture, SmoothingTime, DeltaTime);

		InOutXi = FMath::Abs(TargetX - InOutXi) > DeltaTime * MaxSpeed
			? InOutXi + XDiff * DeltaTime
			: TargetX;
	}

	/** A velocity spring will damp towards a target that follows a fixed linear target velocity, allowing control of the interpolation speed
	* while still giving a smoothed behaviour. A SmoothingTime of 0 will give a linear interpolation between X and TargetX
	 * 
	 * @tparam TVector The type of vector to use (e.g. FVector2D, FVector4, FVector etc)
	 * @param InOutX The value to be damped
	 * @param InOutV The velocity of the value to be damped
	 * @param InOutXi The intermediate target of the value to be damped
	 * @param TargetX The target value of X to damp towards
	 * @param MaxSpeed The max velocity to use for the intermediate target interpolation
	 * @param SmoothingTime The smoothing time to use while damping towards X. Higher values will give more smoothed behaviour. A value of 0 will give a linear interpolation of X to Target
	 * @param DeltaTime The timestep in seconds
	 */
	template <typename TVector>
	static void VelocitySpringDamper(
		TVector& InOutX,
		TVector& InOutV,
		TVector& InOutXi,
		TVector TargetX,
		float MaxSpeed,
		float SmoothingTime,
		float DeltaTime)
	{
		TVector XDiff = TargetX - InOutXi;
		float XDiffLength = XDiff.Length();
		TVector XDiffDir = XDiffLength > FLT_EPSILON
			? XDiff / XDiffLength
			: TVector::ZeroVector;

		float TGoalFuture = SmoothingTime;
		TVector XGoalFuture = XDiffLength > TGoalFuture * MaxSpeed
			? InOutXi + (XDiffDir * MaxSpeed) * TGoalFuture
			: TargetX;

		CriticalSpringDamper(InOutX, InOutV, XGoalFuture, SmoothingTime, DeltaTime);

		InOutXi = XDiffLength > DeltaTime * MaxSpeed
			? InOutXi + XDiffDir * MaxSpeed * DeltaTime
			: TargetX;
	}

	/** Update the position of a character given a target velocity using a simple damped spring
	 * 
	 * @tparam TVector The type of vector to use (e.g. FVector2D, FVector4, FVector etc)
	 * @param InOutPosition The position of the character
	 * @param InOutVelocity The velocity of the character. Needs to be stored and persisted by the caller. Usually initialized to zero and not modified by the caller.
	 * @param InOutAcceleration The acceleration of the character. Needs to be stored and persisted by the caller. Usually initialized to zero and not modified by the caller.
	 * @param TargetVelocity The target velocity of the character.
	 * @param SmoothingTime The time over which to smooth velocity. It takes roughly the smoothing time in order for the character to reach the target velocity.
	 * @param DeltaTime The delta time to tick the character
	 * @param VDeadzone Deadzone for velocity. Current velocity will snap to target velocity when within the deadzone
	 * @param ADeadzone Deadzone for acceleration. Acceleration will snap to zero when within the deadzone
	 */
	template <typename TVector>
	static void SpringCharacterUpdate(
		TVector& InOutPosition,
		TVector& InOutVelocity,
		TVector& InOutAcceleration,
		const TVector& TargetVelocity,
		float SmoothingTime,
		float DeltaTime,
		float VDeadzone = 1e-2f,
		float ADeadzone = 1e-4f)
	{
		float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;
		TVector J0 = InOutVelocity - TargetVelocity;
		TVector J1 = InOutAcceleration + J0 * Y;
		float EyDt = FMath::InvExpApprox(Y * DeltaTime);

		InOutPosition = EyDt * (((-J1) / (Y * Y)) + ((-J0 - J1 * DeltaTime) / Y)) +
			(J1 / (Y * Y)) + J0 / Y + TargetVelocity * DeltaTime + InOutPosition;
		InOutVelocity = EyDt * (J0 + J1 * DeltaTime) + TargetVelocity;
		InOutAcceleration = EyDt * (InOutAcceleration - J1 * Y * DeltaTime);

		if ((TargetVelocity - InOutVelocity).SquaredLength() < FMath::Square(VDeadzone))
		{
			// We reached our target
			InOutVelocity = TargetVelocity;

			if (InOutAcceleration.SquaredLength() < FMath::Square(ADeadzone))
			{
				InOutAcceleration = TVector::ZeroVector;
			}
		}
	}

	/** Gives predicted positions, velocities and accelerations for SpringCharacterUpdate. Useful for generating a predicted trajectory given known initial start conditions.
	 *
	 * @tparam TVector The type of vector to use (e.g. FVector2D, FVector4, FVector etc)
	 * @param OutPredictedPositions ArrayView of output buffer to put the predicted positions. ArrayView should be the same size as PredictCount
	 * @param OutPredictedVelocities ArrayView of output buffer to put the predicted velocities. ArrayView should be the same size as PredictCount
	 * @param OutPredictedAccelerations ArrayView of output buffer to put the predicted accelerations. ArrayView should be the same size as PredictCount
	 * @param PredictCount How many points to predict. Must be greater than 0
	 * @param CurrentPosition The initial position of the character
	 * @param CurrentVelocity The initial velocity of the character
	 * @param CurrentAcceleration The initial acceleration of the character
	 * @param TargetVelocity The target velocity of the character
	 * @param SmoothingTime The smoothing time of the character. It takes roughly the smoothing time in order for the character to reach the target velocity.
	 * @param SecondsPerPredictionStep How much time in between each prediction step.
	 * @param VDeadzone Deadzone for velocity. Current velocity will snap to target velocity when within the deadzone
	 * @param ADeadzone Deadzone for acceleration. Acceleration will snap to zero when within the deadzone
	 */
	template <typename TVector>
	static void SpringCharacterPredict(
		TArrayView<TVector> OutPredictedPositions,
		TArrayView<TVector> OutPredictedVelocities,
		TArrayView<TVector> OutPredictedAccelerations,
		const TVector& CurrentPosition,
		const TVector& CurrentVelocity,
		const TVector& CurrentAcceleration,
		const TVector& TargetVelocity,
		float SmoothingTime,
		float SecondsPerPredictionStep,
		float VDeadzone = 1e-2f,
		float ADeadzone = 1e-4f)
	{
		int32 PredictCount = OutPredictedPositions.Num();
		check(PredictCount > 0);
		check(OutPredictedVelocities.Num() == PredictCount);
		check(OutPredictedAccelerations.Num() == PredictCount);
		
		for (int32 i = 0; i < PredictCount; i++)
		{
			OutPredictedPositions[i] = CurrentPosition;
			OutPredictedVelocities[i] = CurrentVelocity;
			OutPredictedAccelerations[i] = CurrentAcceleration;
		}

		for (int32 i = 0; i < PredictCount; i++)
		{
			const float PredictTime = (float)(i + 1) * SecondsPerPredictionStep; // Note i+1 since we want index 0 to be the first prediction step
			SpringCharacterUpdate(OutPredictedPositions[i], OutPredictedVelocities[i], OutPredictedAccelerations[i], TargetVelocity, SmoothingTime,
				PredictTime, VDeadzone, ADeadzone);
		}
	}

	/** Estimates the time in the future at which a character driven by a critical spring will reach zero acceleration. */
	static inline TOptional<float> SpringCharacterStoppingInflectionTime(const float InitialVelocity, const float InitialAcceleration, const float SmoothingTime)
	{
		const float V = InitialVelocity;
		const float A = InitialAcceleration;
		const float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;
		const float Denom = A + V * Y;
		
		if (FMath::Abs(Denom) < UE_SMALL_NUMBER)
		{
			return TOptional<float>();
		}

		return -V / Denom > 0.0f ? -V / Denom : TOptional<float>();
	}

	/** Estimates an approximate stopping distance from a smoothing time for a character using a simple damped spring */
	static inline float SpringCharacterStoppingDistance(const float InitialVelocity, const float InitialAcceleration, const float SmoothingTime)
	{
		const float V = InitialVelocity;
		const float A = InitialAcceleration;
		const float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;

		if (FMath::Abs(A) < UE_SMALL_NUMBER)
		{
			return 2.0f * FMath::Abs(V) / FMath::Max(Y, UE_SMALL_NUMBER);
		}

		const float I0 = (V * Y * 2 + A) / FMath::Max(Y * Y, UE_SMALL_NUMBER);

		const TOptional<float> T = SpringCharacterStoppingInflectionTime(V, A, SmoothingTime);

		if (!T.IsSet())
		{
			return FMath::Abs(I0);
		}
		else
		{
			const float IT = (FMath::Exp(-Y * T.GetValue()) / FMath::Max(Y * Y, UE_SMALL_NUMBER)) * (T.GetValue() * A * Y + V * Y * (T.GetValue() * Y + 2.0f) + A);
			return FMath::Abs(I0 - IT) + FMath::Abs(IT);
		}
	}

	/** Estimates an approximate stopping time (the time at which the velocity goes below the VelocityThreshold) from a smoothing time and velocity for a character using a simple damped spring */
	static inline float SpringCharacterStoppingTime(const float InitialVelocity, const float SmoothingTime, const float VelocityThreshold = 10.0f)
	{
		const float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;

		const float VRatio = FMath::Abs(VelocityThreshold) / FMath::Max(FMath::Abs(InitialVelocity), UE_SMALL_NUMBER);

		return -(LambertWLower(-VRatio / UE_EULERS_NUMBER) + 1.0f) / FMath::Max(Y, UE_SMALL_NUMBER);
	}

	/** Estimates an approximate spring smoothing time from a stopping distance and velocity for a character using a simple damped spring. Assumes the initial acceleration is zero. */
	static inline float SpringCharacterSmoothingTimeFromStoppingDistance(const float InitialVelocity, const float StoppingDistance)
	{
		return DampingToSmoothingTime(4.0f * FMath::Abs(InitialVelocity) / FMath::Max(StoppingDistance, UE_SMALL_NUMBER));
	}

	/** Estimates an approximate spring smoothing time from a stopping time and velocity for a character using a simple damped spring. Assumes the initial acceleration is zero. */
	static inline float SpringCharacterSmoothingTimeFromStoppingTime(const float InitialVelocity, const float StoppingTime, const float VelocityThreshold = 10.0f)
	{
		const float VRatio = FMath::Clamp(FMath::Abs(VelocityThreshold) / FMath::Max(FMath::Abs(InitialVelocity), UE_SMALL_NUMBER), UE_SMALL_NUMBER, 1.0f - UE_SMALL_NUMBER);

		return DampingToSmoothingTime(2.0f * -(LambertWLower(-VRatio / UE_EULERS_NUMBER) + 1.0f) / FMath::Max(StoppingTime, UE_SMALL_NUMBER));
	}

	/** Estimates an approximate starting time (the time at which the velocity gets within the VelocityThreshold of the TargetVelocity) from a smoothing time and target velocity for a character using a simple damped spring */
	static inline float SpringCharacterStartingTime(const float TargetVelocity, const float SmoothingTime, const float VelocityThreshold = 10.0f)
	{
		const float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;

		// Starting time is the same as if the character was moving at the target velocity and coming to a stop
		const float VRatio = FMath::Abs(VelocityThreshold) / FMath::Max(FMath::Abs(TargetVelocity), UE_SMALL_NUMBER);

		return -(LambertWLower(-VRatio / UE_EULERS_NUMBER) + 1.0f) / FMath::Max(Y, UE_SMALL_NUMBER);
	}

	/** Estimates an approximate starting distance (the distance traveled until the velocity gets within the VelocityThreshold of the TargetVelocity) from a smoothing time for a character using a simple damped spring */
	static inline float SpringCharacterStartingDistance(const float TargetVelocity, const float SmoothingTime, const float VelocityThreshold = 10.0f)
	{
		const float V = FMath::Abs(TargetVelocity);
		const float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;
		const float T = SpringCharacterStartingTime(V, SmoothingTime, VelocityThreshold);

		return FMath::Exp(-Y * T) * ((2.0f * V) / FMath::Max(Y, UE_SMALL_NUMBER) + V * T) - (2.0f * V) / FMath::Max(Y, UE_SMALL_NUMBER) + V * T;
	}

	/** Estimates an approximate spring smoothing time from a starting time and target velocity for a character using a simple damped spring. */
	static inline float SpringCharacterSmoothingTimeFromStartingTime(const float TargetVelocity, const float StartingTime, const float VelocityThreshold = 10.0f)
	{
		// Smoothing time is the same as if the character was moving at the target velocity and coming to a stop
		return SpringCharacterSmoothingTimeFromStoppingTime(TargetVelocity, StartingTime, VelocityThreshold);
	}

	/** Estimates an approximate spring smoothing time from a starting distance and target velocity for a character using a simple damped spring. */
	static inline float SpringCharacterSmoothingTimeFromStartingDistance(const float TargetVelocity, const float StartingDistance, const float VelocityThreshold = 10.0f)
	{
		const float VRatio = FMath::Abs(VelocityThreshold) / FMath::Max(FMath::Abs(TargetVelocity), UE_SMALL_NUMBER);

		const float W = -(LambertWLower(-VRatio / UE_EULERS_NUMBER) + 1.0f);

		const float V = FMath::Abs(TargetVelocity);
		return DampingToSmoothingTime((2.0f * FMath::Exp(-W) * V * (2.0f + W) + 2.0f * V * (W - 2.0f)) / FMath::Max(StartingDistance, UE_SMALL_NUMBER));
	}

	/** Estimates the time at which the maximum acceleration will occur. */
	static inline TOptional<float> SpringCharacterMaximumAccelerationTime(const float InitialVelocity, const float InitialAcceleration, const float TargetVelocity, const float SmoothingTime)
	{
		const float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;
		const float J0 = InitialVelocity - TargetVelocity;
		const float J1 = InitialAcceleration + J0 * Y;
		const float Denom = J1 * Y;
		if (FMath::Abs(Denom) < UE_SMALL_NUMBER)
		{
			return TOptional<float>();
		}

		return (J1 + InitialAcceleration) / Denom > 0.0f ? (J1 + InitialAcceleration) / Denom : TOptional<float>();
	}

	/** Estimates the maximum acceleration for a character */
	static inline float SpringCharacterMaximumAcceleration(const float InitialVelocity, const float InitialAcceleration, const float TargetVelocity, const float SmoothingTime)
	{
		const float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;
		const float J0 = InitialVelocity - TargetVelocity;
		const float J1 = InitialAcceleration + J0 * Y;
		const TOptional<float> T = SpringCharacterMaximumAccelerationTime(InitialVelocity, InitialAcceleration, TargetVelocity, SmoothingTime);

		if (T.IsSet())
		{
			return FMath::Max(FMath::Abs(InitialAcceleration), FMath::Abs(FMath::Exp(-Y * T.GetValue()) * (InitialAcceleration - J1 * Y * T.GetValue())));
		}
		else
		{
			return FMath::Abs(InitialAcceleration);
		}
	}

	/** Estimates an approximate spring smoothing time from the maximum acceleration */
	static inline float SpringCharacterSmoothingTimeFromMaximumAcceleration(const float InitialVelocity, const float TargetVelocity, const float MaximumAcceleration)
	{
		return DampingToSmoothingTime(2.0f * ((FMath::Abs(MaximumAcceleration) / FMath::Max(FMath::Abs(TargetVelocity - InitialVelocity), UE_SMALL_NUMBER)) * UE_EULERS_NUMBER));
	}

	/** Estimates the time in the future at which the maximum angular velocity will occur. */
	static inline TOptional<float> SpringCharacterMaximumAngularVelocityTime(const float InitialAngle, const float InitialAngularVelocity, const float TargetAngle, const float SmoothingTime)
	{
		const float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;
		const float J0 = FMath::FindDeltaAngleRadians(TargetAngle, InitialAngle);
		const float J1 = InitialAngularVelocity + J0 * Y;
		const float Denom = J1 * Y;
		if (FMath::Abs(Denom) < UE_SMALL_NUMBER)
		{
			return TOptional<float>();
		}

		return (J1 + InitialAngularVelocity) / Denom > 0.0f ? (J1 + InitialAngularVelocity) / Denom : TOptional<float>();
	}

	/** Estimates the maximum angular velocity for a character */
	static inline float SpringCharacterMaximumAngularVelocity(const float InitialAngle, const float InitialAngularVelocity, const float TargetAngle, const float SmoothingTime)
	{
		const float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;
		const float J0 = FMath::FindDeltaAngleRadians(TargetAngle, InitialAngle);
		const float J1 = InitialAngularVelocity + J0 * Y;
		const TOptional<float> T = SpringCharacterMaximumAngularVelocityTime(InitialAngle, InitialAngularVelocity, TargetAngle, SmoothingTime);

		if (T.IsSet())
		{
			return FMath::Max(FMath::Abs(InitialAngularVelocity), FMath::Abs(FMath::Exp(-Y * T.GetValue()) * (InitialAngularVelocity - J1 * Y * T.GetValue())));
		}
		else
		{
			return FMath::Abs(InitialAngularVelocity);
		}
	}

	/** Estimates an approximate spring smoothing time from the maximum angular velocity */
	static inline float SpringCharacterSmoothingTimeFromMaximumAngularVelocity(const float InitialAngle, const float TargetAngle, const float MaximumAngularVelocity)
	{
		return DampingToSmoothingTime(2.0f * ((FMath::Abs(MaximumAngularVelocity) / FMath::Max(FMath::Abs(FMath::FindDeltaAngleRadians(InitialAngle, TargetAngle)), UE_SMALL_NUMBER)) * UE_EULERS_NUMBER));
	}

	/** Update a position representing a character given a target velocity using a velocity spring.
	 * A velocity spring tracks an intermediate velocity which moves at a maximum acceleration linearly towards a target.
	 * This means unlike the "SpringCharacterUpdate", it will take longer to reach a target velocity that is further away from the current velocity.
	 * 
	 * @tparam TVector The type of vector to use (e.g. FVector2D, FVector4, FVector etc)
	 * @param InOutPosition The position of the character
	 * @param InOutVelocity The velocity of the character. Needs to be stored and persisted by the caller. Usually initialized to zero and not modified by the caller.
	 * @param InOutVelocityIntermediate The intermediate velocity of the character. Needs to be stored and persisted by the caller. Usually initialized to zero and not modified by the caller.
	 * @param InOutAcceleration The acceleration of the character. Needs to be stored and persisted by the caller. Usually initialized to zero and not modified by the caller.
	 * @param TargetVelocity The target velocity of the character.
	 * @param SmoothingTime The time over which to smooth velocity. It takes roughly the smoothing time in order for the character to reach the target velocity.
	 * @param MaxAcceleration Puts a limit on the maximum acceleration that the intermediate velocity can do each frame. If MaxAccel is very large, the behaviour wil lbe the same as SpringCharacterUpdate
	 * @param DeltaTime The delta time to tick the character
	 * @param VDeadzone Deadzone for velocity. Current velocity will snap to target velocity when within the deadzone
	 * @param ADeadzone Deadzone for acceleration. Acceleration will snap to zero when within the deadzone
	 */
	template <typename TVector>
	static void VelocitySpringCharacterUpdate(
		TVector& InOutPosition,
		TVector& InOutVelocity,
		TVector& InOutVelocityIntermediate,
		TVector& InOutAcceleration,
		TVector TargetVelocity,
		float SmoothingTime,
		float MaxAcceleration,
		float DeltaTime,
		float VDeadzone = 1e-2f,
		float ADeadzone = 1e-4f)
	{
		TVector VDiff = TargetVelocity - InOutVelocityIntermediate;
		float VDiffLength = VDiff.Length();
		TVector VDiffDir = VDiffLength > 0.0001f
			? VDiff / VDiffLength
			: TVector::ZeroVector;

		float TGoalFuture = SmoothingTime;
		TVector MaxVFuture = VDiffLength > TGoalFuture * MaxAcceleration
			? InOutVelocityIntermediate + (VDiffDir * MaxAcceleration) * TGoalFuture
			: TargetVelocity;

		float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;
		TVector J0 = InOutVelocity - MaxVFuture;
		TVector J1 = InOutAcceleration + J0 * Y;
		float EyDt = FMath::InvExpApprox(Y * DeltaTime);

		InOutPosition = EyDt * (((-J1) / (Y * Y)) + ((-J0 - J1 * DeltaTime) / Y)) +
			(J1 / (Y * Y)) + J0 / Y + MaxVFuture * DeltaTime + InOutPosition;
		InOutVelocity = EyDt * (J0 + J1 * DeltaTime) + MaxVFuture;
		InOutAcceleration = EyDt * (InOutAcceleration - J1 * Y * DeltaTime);
		InOutVelocityIntermediate = VDiffLength > DeltaTime * MaxAcceleration
			? InOutVelocityIntermediate + VDiffDir * MaxAcceleration * DeltaTime
			: TargetVelocity;

		if ((TargetVelocity - InOutVelocity).SquaredLength() < FMath::Square(VDeadzone))
		{
			// We reached our target
			InOutVelocity = TargetVelocity;

			if (InOutAcceleration.SquaredLength() < FMath::Square(ADeadzone))
			{
				InOutAcceleration = TVector::ZeroVector;
			}
		}
	}

	/** Gives predicted positions, velocities and accelerations for SpringCharacterUpdate. Useful for generating a predicted trajectory given known initial start conditions.
	 *
	 * @tparam TVector The type of vector to use (e.g. FVector2D, FVector4, FVector etc)
	 * @param OutPredictedPositions ArrayView of output buffer to put the predicted positions. ArrayView should be the same size as PredictCount
	 * @param OutPredictedVelocities ArrayView of output buffer to put the predicted velocities. ArrayView should be the same size as PredictCount
	 * @param OutPredictedIntermediateVelocities ArrayView of output buffer to put the predicted intermediate velocities. ArrayView should be the same size as PredictCount
	 * @param OutPredictedAccelerations ArrayView of output buffer to put the predicted accelerations. ArrayView should be the same size as PredictCount
	 * @param PredictCount How many points to predict. Must be greater than 0
	 * @param CurrentPosition The initial position of the character
	 * @param CurrentVelocity The initial velocity of the character
	 * @param CurrentIntermediateVelocity The initial intermediate velocity of the character
	 * @param CurrentAcceleration The initial acceleration of the character
	 * @param TargetVelocity The target velocity of the character
	 * @param SmoothingTime The smoothing time of the character. It takes roughly the smoothing time in order for the character to reach the target velocity.
	 * @param MaxAcceleration Puts a limit on the maximum acceleration that the intermediate velocity can do each frame. If MaxAccel is very large, the behaviour wil lbe the same as SpringCharacterUpdate
	 * @param SecondsPerPredictionStep How much time in between each prediction step.
	 * @param VDeadzone Deadzone for velocity. Current velocity will snap to target velocity when within the deadzone
	 * @param ADeadzone Deadzone for acceleration. Acceleration will snap to zero when within the deadzone
	 */
	template <typename TVector>
	static void VelocitySpringCharacterPredict(
		TArrayView<TVector> OutPredictedPositions,
		TArrayView<TVector> OutPredictedVelocities,
		TArrayView<TVector> OutPredictedIntermediateVelocities,
		TArrayView<TVector> OutPredictedAccelerations,
		const TVector& CurrentPosition,
		const TVector& CurrentVelocity,
		const TVector& CurrentIntermediateVelocity,
		const TVector& CurrentAcceleration,
		const TVector& TargetVelocity,
		float SmoothingTime,
		float MaxAcceleration,
		float SecondsPerPredictionStep,
		float VDeadzone = 1e-2f,
		float ADeadzone = 1e-4f)
	{
		int32 PredictCount = OutPredictedPositions.Num();
		check(PredictCount > 0);
		check(OutPredictedVelocities.Num() == PredictCount);
		check(OutPredictedAccelerations.Num() == PredictCount);
		check(OutPredictedIntermediateVelocities.Num() == PredictCount);
		
		for (int32 i = 0; i < PredictCount; i++)
		{
			OutPredictedPositions[i] = CurrentPosition;
			OutPredictedVelocities[i] = CurrentVelocity;
			OutPredictedIntermediateVelocities[i] = CurrentIntermediateVelocity;
			OutPredictedAccelerations[i] = CurrentAcceleration;
		}

		for (int32 i = 0; i < PredictCount; i++)
		{
			const float PredictTime = (float)(i + 1) * SecondsPerPredictionStep; // Note i+1 since we want index 0 to be the first prediction step
			VelocitySpringCharacterUpdate(OutPredictedPositions[i],
				OutPredictedVelocities[i],
				OutPredictedIntermediateVelocities[i],
				OutPredictedAccelerations[i],
				TargetVelocity,
				SmoothingTime,
				MaxAcceleration,
				PredictTime,
				VDeadzone,
				ADeadzone);
		}
	}

	/** Prediction of CriticalSpringDamperQuat
	 * 
	 * @param OutPredictedRotations ArrayView of output buffer to put the predicted rotations. ArrayView should be the same size as PredictCount
	 * @param OutPredictedAngularVelocities ArrayView of output buffer to put the predicted angular velocities. ArrayView should be the same size as PredictCount
	 * @param PredictCount How many points to predict. Must be greater than 0
	 * @param CurrentRotation Initial rotation at t = 0
	 * @param CurrentAngularVelocity Initial angular velocity at t = 0
	 * @param TargetRotation The target rotation
	 * @param SmoothingTime The smoothing time
	 * @param SecondsPerPredictionStep How many seconds per prediction step
	 */
	static void CriticalSpringDamperQuatPredict(TArrayView<FQuat> OutPredictedRotations, TArrayView<FVector> OutPredictedAngularVelocities,
	                                          int32 PredictCount, const FQuat& CurrentRotation, const FVector& CurrentAngularVelocity,
	                                          const FQuat& TargetRotation, float SmoothingTime, float SecondsPerPredictionStep)
	{
		check(OutPredictedRotations.Num() == PredictCount);
		check(OutPredictedAngularVelocities.Num() == PredictCount);
		
		for (int32 i = 0; i < PredictCount; i++)
		{
			OutPredictedRotations[i] = CurrentRotation;
			OutPredictedAngularVelocities[i] = CurrentAngularVelocity;
		}

		for (int32 i = 0; i < PredictCount; i++)
		{
			const float PredictTime = (float)(i + 1) * SecondsPerPredictionStep; // Note i+1 since we want index 0 to be the first prediction step
			CriticalSpringDamperQuat(OutPredictedRotations[i], OutPredictedAngularVelocities[i], TargetRotation, SmoothingTime, PredictTime);
		}
	}

	/** Specialized quaternion damper, similar to FMath::ExponentialSmoothingApprox but for quaternions.
	 *  Smooths a value using exponential damping towards a target.
	 * 
	 * @param InOutRotation			The value to be smoothed
	 * @param InTargetRotation		The target to smooth towards
	 * @param InDeltaTime			Time interval
	 * @param SmoothingTime			Timescale over which to smooth. Larger values result in more smoothed behaviour. Can be zero.
	 */
	static constexpr void ExponentialSmoothingApproxQuat(FQuat& InOutRotation, const FQuat& InTargetRotation, const float InDeltaTime, const float SmoothingTime)
	{
		if (SmoothingTime > UE_SMALL_NUMBER)
		{
			InOutRotation = FQuat::Slerp(InOutRotation, InTargetRotation, 1.0f - FMath::InvExpApprox(InDeltaTime / SmoothingTime));
		}
		else
		{
			InOutRotation = InTargetRotation;
		}
	}

	/** Specialized angle damper, similar to FMath::ExponentialSmoothingApprox but deals correctly with angle wrap-around.
	 *  Smooths an angle using exponential damping towards a target.
	 *
	 * @param InOutAngleRadians		The angle to be smoothed
	 * @param InTargetAngleRadians	The target to smooth towards
	 * @param InDeltaTime			Time interval
	 * @param SmoothingTime			Timescale over which to smooth. Larger values result in more smoothed behaviour. Can be zero.
	 */
	static constexpr void ExponentialSmoothingApproxAngle(float& InOutAngleRadians, const float& InTargetAngleRadians, const float InDeltaTime, const float SmoothingTime)
	{
		if (SmoothingTime > UE_SMALL_NUMBER)
		{
			InOutAngleRadians += FMath::FindDeltaAngleRadians(InOutAngleRadians, InTargetAngleRadians) * (1.0f - FMath::InvExpApprox(InDeltaTime / SmoothingTime));
		}
		else
		{
			InOutAngleRadians = InTargetAngleRadians;
		}
	}

	/** Specialized scale damper, similar to FMath::ExponentialSmoothingApprox but for scales.
	 *  Smooths a value using exponential damping towards a target.
	 *
	 * @param InOutScale			The value to be smoothed
	 * @param InTargetScale			The target to smooth towards
	 * @param InDeltaTime			Time interval
	 * @param SmoothingTime			Timescale over which to smooth. Larger values result in more smoothed behaviour. Can be zero.
	 */
	static constexpr void ExponentialSmoothingApproxScale(FVector& InOutScale, const FVector& InTargetScale, const float InDeltaTime, const float SmoothingTime)
	{
		if (SmoothingTime > UE_SMALL_NUMBER)
		{
			InOutScale = ScaleEerp(InOutScale, InTargetScale, 1.0f - FMath::InvExpApprox(InDeltaTime / SmoothingTime));
		}
		else
		{
			InOutScale = InTargetScale;
		}
	}

	/** Tracks the velocity of the input value InValue via finite difference, updating InOutValue and InOutVelocity in place.
	 *
	 * @param InOutValue			Updates to be the same as InValue
	 * @param InOutVelocity			Updates to be the velocity of InValue computed via finite difference
	 * @param InValue				Input value
	 * @param InDeltaTime			Time interval
	 */
	template <typename TValue>
	static constexpr void TrackVelocity(TValue& InOutValue, TValue& InOutVelocity, const TValue& InValue, float DeltaTime)
	{
		InOutVelocity = DeltaTime > UE_SMALL_NUMBER ? (InValue - InOutValue) / DeltaTime : TValue(0.0);
		InOutValue = InValue;
	}

	/** Tracks the velocity of the input rotation InValue via finite difference, updating InOutValue and InOutVelocity in place.
	 *
	 * @param InOutValue			Updates to be the same as InValue
	 * @param InOutVelocity			Updates to be the velocity of InValue computed via finite difference (in rad/s)
	 * @param InValue				Input rotation
	 * @param InDeltaTime			Time interval
	 */
	static constexpr void TrackVelocityQuat(FQuat& InOutValue, FVector& InOutVelocity, FQuat InValue, float DeltaTime)
	{
		InOutVelocity = DeltaTime > UE_SMALL_NUMBER ? (InValue * InOutValue.Inverse()).GetShortestArcWith(FQuat::Identity).ToRotationVector() / DeltaTime : FVector::ZeroVector;
		InOutValue = InValue;
	}

	/** Tracks the velocity of the input angle InValue via finite difference, updating InOutValue and InOutVelocity in place.
	 *
	 * @param InOutValue			Updates to be the same as InValue
	 * @param InOutVelocity			Updates to be the velocity of InValue computed via finite difference (in rad/s)
	 * @param InValue				Input angle
	 * @param InDeltaTime			Time interval
	 */
	static constexpr void TrackVelocityAngle(float& InOutValue, float& InOutVelocity, float InValue, float DeltaTime)
	{
		InOutVelocity = DeltaTime > UE_SMALL_NUMBER ? FMath::FindDeltaAngleRadians(InOutValue, InValue) / DeltaTime : 0.0f;
		InOutValue = InValue;
	}

	/** Tracks the velocity of the input scale InValue via finite difference, updating InOutValue and InOutVelocity in place.
	 *
	 * @param InOutValue			Updates to be the same as InValue
	 * @param InOutVelocity			Updates to be the velocity of InValue computed via finite difference
	 * @param InValue				Input scale
	 * @param InDeltaTime			Time interval
	 */
	static constexpr void TrackVelocityScale(FVector& InOutValue, FVector& InOutVelocity, FVector InValue, float DeltaTime)
	{
		InOutVelocity = DeltaTime > UE_SMALL_NUMBER ? ScaleLog(ScaleDivMax(InValue, InOutValue)) / DeltaTime : FVector::ZeroVector;
		InOutValue = InValue;
	}

	/** Applies a spring inertialization to the input value (and velocity), updating them in place by adding the appropriate offset given the time
	 * since transition.
	 *
	 * @param InOutValue			The value to apply the inertialization to
	 * @param InOutVelocity			The velocity to apply the inertialization to
	 * @param InValueOffset			Offset value at the time of transition
	 * @param InVelocityOffset		Offset velocity at the time of transition
	 * @param TimeSinceTransition	Time since the last transition
	 * @param SmoothingTime			Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	template <typename TValue>
	static void SpringInertializeApply(
		TValue& InOutValue,
		TValue& InOutVelocity,
		const TValue& InValueOffset,
		const TValue& InVelocityOffset,
		float TimeSinceTransition,
		float SmoothingTime)
	{
		TValue ValueOffset = InValueOffset;
		TValue VelocityOffset = InVelocityOffset;

		CriticalDecay<TValue>(ValueOffset, VelocityOffset, SmoothingTime, TimeSinceTransition);

		InOutValue += ValueOffset;
		InOutVelocity += VelocityOffset;
	}

	/** Transitions a spring inertialization, computing a new offset value.
	 *
	 * @param InOutValueOffset		Offset value to update
	 * @param InOutVelocityOffset	Offset velocity to update
	 * @param InSrcValue			Value of the source (value transitioning from)
	 * @param InSrcVelocity			Velocity of the source (velocity transitioning from)
	 * @param InDstValue			Value of the destination (value transitioning to)
	 * @param InDstVelocity			Velocity of the destination (velocity transitioning to)
	 * @param TimeSinceTransition	Time since the last transition
	 * @param SmoothingTime			Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	template <typename TValue>
	static void SpringInertializeTransition(
		TValue& InOutValueOffset,
		TValue& InOutVelocityOffset,
		const TValue& InSrcValue,
		const TValue& InSrcVelocity,
		const TValue& InDstValue,
		const TValue& InDstVelocity,
		float TimeSinceTransition,
		float SmoothingTime)
	{
		TValue ValueOffset = InOutValueOffset;
		TValue VelocityOffset = InOutVelocityOffset;

		CriticalDecay<TValue>(ValueOffset, VelocityOffset, SmoothingTime, TimeSinceTransition);

		InOutValueOffset = InSrcValue + ValueOffset - InDstValue;
		InOutVelocityOffset = InSrcVelocity + VelocityOffset - InDstVelocity;
	}

	/** Specialization of SpringInertializeApply for quaternions. Stores velocities in rad/s.
	 *
	 * @param InOutValue			The value to apply the inertialization to
	 * @param InOutVelocity			The velocity to apply the inertialization to
	 * @param InValueOffset			Offset value at the time of transition
	 * @param InVelocityOffset		Offset velocity at the time of transition
	 * @param TimeSinceTransition	Time since the last transition
	 * @param SmoothingTime			Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	static void SpringInertializeApplyQuat(
		FQuat& InOutRotation,
		FVector& InOutAngularVelocity,
		const FQuat& InRotationOffset,
		const FVector& InAngularVelocityOffset,
		float TimeSinceTransition,
		float SmoothingTime)
	{
		FQuat RotationOffset = InRotationOffset;
		FVector AngularVelocityOffset = InAngularVelocityOffset;

		CriticalDecayQuat(RotationOffset, AngularVelocityOffset, SmoothingTime, TimeSinceTransition);

		InOutRotation = RotationOffset * InOutRotation;
		InOutAngularVelocity += AngularVelocityOffset;
	}

	/** Specialization of SpringInertializeTransition for quaternions. Stores velocities in rad/s.
	 *
	 * @param InOutValueOffset		Offset value to update
	 * @param InOutVelocityOffset	Offset velocity to update
	 * @param InSrcValue			Value of the source (value transitioning from)
	 * @param InSrcVelocity			Velocity of the source (velocity transitioning from)
	 * @param InDstValue			Value of the destination (value transitioning to)
	 * @param InDstVelocity			Velocity of the destination (velocity transitioning to)
	 * @param TimeSinceTransition	Time since the last transition
	 * @param SmoothingTime			Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	static void SpringInertializeTransitionQuat(
		FQuat& InOutRotationOffset,
		FVector& InOutAngularVelocityOffset,
		const FQuat& InSrcRotation,
		const FVector& InSrcAngularVelocity,
		const FQuat& InDstRotation,
		const FVector& InDstAngularVelocity,
		float TimeSinceTransition,
		float SmoothingTime)
	{
		FQuat RotationOffset = InOutRotationOffset;
		FVector AngularVelocityOffset = InOutAngularVelocityOffset;

		CriticalDecayQuat(RotationOffset, AngularVelocityOffset, SmoothingTime, TimeSinceTransition);

		InOutRotationOffset = InSrcRotation * RotationOffset * InDstRotation.Inverse();
		InOutAngularVelocityOffset = InSrcAngularVelocity + AngularVelocityOffset - InDstAngularVelocity;
	}

	/** Specialization of SpringInertializeApply for angles. Stores velocities in rad/s.
	 *
	 * @param InOutValue			The value to apply the inertialization to
	 * @param InOutVelocity			The velocity to apply the inertialization to
	 * @param InValueOffset			Offset value at the time of transition
	 * @param InVelocityOffset		Offset velocity at the time of transition
	 * @param TimeSinceTransition	Time since the last transition
	 * @param SmoothingTime			Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	static void SpringInertializeApplyAngle(
		float& InOutAngle,
		float& InOutAngularVelocity,
		float InAngleOffset,
		float InAngularVelocityOffset,
		float TimeSinceTransition,
		float SmoothingTime)
	{
		float AngleOffset = InAngleOffset;
		float AngularVelocityOffset = InAngularVelocityOffset;

		CriticalDecayAngle(AngleOffset, AngularVelocityOffset, SmoothingTime, TimeSinceTransition);

		InOutAngle += AngleOffset;
		InOutAngularVelocity += AngularVelocityOffset;
	}

	/** Specialization of SpringInertializeTransition for angles. Stores velocities in rad/s.
	 *
	 * @param InOutValueOffset		Offset value to update
	 * @param InOutVelocityOffset	Offset velocity to update
	 * @param InSrcValue			Value of the source (value transitioning from)
	 * @param InSrcVelocity			Velocity of the source (velocity transitioning from)
	 * @param InDstValue			Value of the destination (value transitioning to)
	 * @param InDstVelocity			Velocity of the destination (velocity transitioning to)
	 * @param TimeSinceTransition	Time since the last transition
	 * @param SmoothingTime			Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	static void SpringInertializeTransitionAngle(
		float& InOutAngleOffset,
		float& InOutAngularVelocityOffset,
		float InSrcAngle,
		float InSrcAngularVelocity,
		float InDstAngle,
		float InDstAngularVelocity,
		float TimeSinceTransition,
		float SmoothingTime)
	{
		float AngleOffset = InOutAngleOffset;
		float AngularVelocityOffset = InOutAngularVelocityOffset;

		CriticalDecayAngle(AngleOffset, AngularVelocityOffset, SmoothingTime, TimeSinceTransition);

		InOutAngleOffset = FMath::FindDeltaAngleRadians(InDstAngle, InSrcAngle + AngleOffset);
		InOutAngularVelocityOffset = InSrcAngularVelocity + AngularVelocityOffset - InDstAngularVelocity;
	}

	/** Specialization of SpringInertializeApply for scales.
	 *
	 * @param InOutValue			The value to apply the inertialization to
	 * @param InOutVelocity			The velocity to apply the inertialization to
	 * @param InValueOffset			Offset value at the time of transition
	 * @param InVelocityOffset		Offset velocity at the time of transition
	 * @param TimeSinceTransition	Time since the last transition
	 * @param SmoothingTime			Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	static void SpringInertializeApplyScale(
		FVector& InOutScale,
		FVector& InOutScalarVelocity,
		const FVector& InScaleOffset,
		const FVector& InScalarVelocityOffset,
		float TimeSinceTransition,
		float SmoothingTime)
	{
		FVector ScaleOffset = InScaleOffset;
		FVector ScalarVelocityOffset = InScalarVelocityOffset;

		CriticalDecayScale(ScaleOffset, ScalarVelocityOffset, SmoothingTime, TimeSinceTransition);

		InOutScale = ScaleOffset * InOutScale;
		InOutScalarVelocity += ScalarVelocityOffset;
	}

	/** Specialization of SpringInertializeTransition for Scales
	 *
	 * @param InOutValueOffset		Offset value to update
	 * @param InOutVelocityOffset	Offset velocity to update
	 * @param InSrcValue			Value of the source (value transitioning from)
	 * @param InSrcVelocity			Velocity of the source (velocity transitioning from)
	 * @param InDstValue			Value of the destination (value transitioning to)
	 * @param InDstVelocity			Velocity of the destination (velocity transitioning to)
	 * @param TimeSinceTransition	Time since the last transition
	 * @param SmoothingTime			Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	static void SpringInertializeTransitionScale(
		FVector& InOutScaleOffset,
		FVector& InOutScalarVelocityOffset,
		const FVector& InSrcScale,
		const FVector& InSrcScalarVelocity,
		const FVector& InDstScale,
		const FVector& InDstScalarVelocity,
		float TimeSinceTransition,
		float SmoothingTime)
	{
		FVector ScaleOffset = InOutScaleOffset;
		FVector ScalarVelocityOffset = InOutScalarVelocityOffset;

		CriticalDecayScale(ScaleOffset, ScalarVelocityOffset, SmoothingTime, TimeSinceTransition);

		InOutScaleOffset = ScaleDivMax(InSrcScale * ScaleOffset, InDstScale);
		InOutScalarVelocityOffset = InSrcScalarVelocity + ScalarVelocityOffset - InDstScalarVelocity;
	}

	/** Compute the cubic weights required to inertialize out an offset and velocity offset for a given time and blend time */
	static inline void CubicDecayWeights(float& OutW0, float& OutW1, float& OutW2, float& OutW3, const float Time, const float BlendTime)
	{
		float NonZeroBlendTime = FMath::Max(BlendTime, UE_SMALL_NUMBER);
		float T = FMath::Clamp(Time / NonZeroBlendTime, 0.0f, 1.0f);
		OutW0 = 2.0f * T * T * T - 3.0f * T * T + 1.0f;
		OutW1 = (T * T * T - 2.0f * T * T + T) * BlendTime;
		OutW2 = (6.0f * T * T - 6.0f * T) / NonZeroBlendTime;
		OutW3 = 3.0f * T * T - 4.0f * T + 1.0f;
	}

	/** Applies a cubic inertialization to the input value (and velocity), updating them in place by adding the appropriate offset given the time
	 * since transition.
	 *
	 * @param InOutValue			The value to apply the inertialization to
	 * @param InOutVelocity			The velocity to apply the inertialization to
	 * @param InValueOffset			Offset value at the time of transition
	 * @param InVelocityOffset		Offset velocity at the time of transition
	 * @param TimeSinceTransition	Time since the last transition
	 * @param BlendTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	template <typename TValue>
	static void CubicInertializeApply(
		TValue& InOutValue,
		TValue& InOutVelocity,
		const TValue& InValueOffset,
		const TValue& InVelocityOffset,
		float TimeSinceTransition,
		float BlendTime)
	{
		float W0 = 0.0f, W1 = 0.0f, W2 = 0.0f, W3 = 0.0f;
		CubicDecayWeights(W0, W1, W2, W3, TimeSinceTransition, BlendTime);

		InOutValue += W0 * InValueOffset + W1 * InVelocityOffset;
		InOutVelocity += W2 * InValueOffset + W3 * InVelocityOffset;
	}

	/** Transitions a cubic inertialization, computing a new offset value.
	 *
	 * @param InOutValueOffset		Offset value to update
	 * @param InOutVelocityOffset	Offset velocity to update
	 * @param InSrcValue			Value of the source (value transitioning from)
	 * @param InSrcVelocity			Velocity of the source (velocity transitioning from)
	 * @param InDstValue			Value of the destination (value transitioning to)
	 * @param InDstVelocity			Velocity of the destination (velocity transitioning to)
	 * @param TimeSinceTransition	Time since the last transition
	 * @param BlendTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	template <typename TValue>
	static void CubicInertializeTransition(
		TValue& InOutValueOffset,
		TValue& InOutVelocityOffset,
		const TValue& InSrcValue,
		const TValue& InSrcVelocity,
		const TValue& InDstValue,
		const TValue& InDstVelocity,
		float TimeSinceTransition,
		float BlendTime)
	{
		float W0 = 0.0f, W1 = 0.0f, W2 = 0.0f, W3 = 0.0f;
		CubicDecayWeights(W0, W1, W2, W3, TimeSinceTransition, BlendTime);

		TValue Value = InSrcValue + W0 * InOutValueOffset + W1 * InOutVelocityOffset;
		TValue Velocity = InSrcVelocity + W2 * InOutValueOffset + W3 * InOutVelocityOffset;

		InOutValueOffset = Value - InDstValue;
		InOutVelocityOffset = Velocity - InDstVelocity;
	}

	/** Specialization of CubicInertializeApply for quaternions. Stores velocities in rad/s.
	 *
	 * @param InOutValue			The value to apply the inertialization to
	 * @param InOutVelocity			The velocity to apply the inertialization to
	 * @param InValueOffset			Offset value at the time of transition
	 * @param InVelocityOffset		Offset velocity at the time of transition
	 * @param TimeSinceTransition	Time since the last transition
	 * @param BlendTime				Blend over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	static void CubicInertializeApplyQuat(
		FQuat& InOutRotation,
		FVector& InOutAngularVelocity,
		const FQuat& InRotationOffset,
		const FVector& InAngularVelocityOffset,
		float TimeSinceTransition,
		float BlendTime)
	{
		float W0 = 0.0f, W1 = 0.0f, W2 = 0.0f, W3 = 0.0f;
		CubicDecayWeights(W0, W1, W2, W3, TimeSinceTransition, BlendTime);

		InOutRotation = FQuat::MakeFromRotationVector(W0 * InRotationOffset.ToRotationVector() + W1 * InAngularVelocityOffset) * InOutRotation;
		InOutAngularVelocity += W2 * InRotationOffset.ToRotationVector() + W3 * InAngularVelocityOffset;
	}

	/** Specialization of CubicInertializeTransition for quaternions. Stores velocities in rad/s.
	 *
	 * @param InOutValueOffset		Offset value to update
	 * @param InOutVelocityOffset	Offset velocity to update
	 * @param InSrcValue			Value of the source (value transitioning from)
	 * @param InSrcVelocity			Velocity of the source (velocity transitioning from)
	 * @param InDstValue			Value of the destination (value transitioning to)
	 * @param InDstVelocity			Velocity of the destination (velocity transitioning to)
	 * @param TimeSinceTransition	Time since the last transition
	 * @param BlendTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	static void CubicInertializeTransitionQuat(
		FQuat& InOutRotationOffset,
		FVector& InOutAngularVelocityOffset,
		const FQuat& InSrcRotation,
		const FVector& InSrcAngularVelocity,
		const FQuat& InDstRotation,
		const FVector& InDstAngularVelocity,
		float TimeSinceTransition,
		float BlendTime)
	{
		float W0 = 0.0f, W1 = 0.0f, W2 = 0.0f, W3 = 0.0f;
		CubicDecayWeights(W0, W1, W2, W3, TimeSinceTransition, BlendTime);

		FQuat Value = FQuat::MakeFromRotationVector(W0 * InOutRotationOffset.ToRotationVector() + W1 * InOutAngularVelocityOffset) * InSrcRotation;
		FVector Velocity = InSrcAngularVelocity + W2 * InOutRotationOffset.ToRotationVector() + W3 * InOutAngularVelocityOffset;

		InOutRotationOffset = (Value * InDstRotation.Inverse()).GetShortestArcWith(FQuat::Identity);
		InOutAngularVelocityOffset = Velocity - InDstAngularVelocity;
	}

	/** Specialization of CubicInertializeApply for angles. Stores velocities in rad/s.
	 *
	 * @param InOutValue			The value to apply the inertialization to
	 * @param InOutVelocity			The velocity to apply the inertialization to
	 * @param InValueOffset			Offset value at the time of transition
	 * @param InVelocityOffset		Offset velocity at the time of transition
	 * @param TimeSinceTransition	Time since the last transition
	 * @param BlendTime				Blend over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	static void CubicInertializeApplyAngle(
		float& InOutAngle,
		float& InOutAngularVelocity,
		float InAngleOffset,
		float InAngularVelocityOffset,
		float TimeSinceTransition,
		float BlendTime)
	{
		float W0 = 0.0f, W1 = 0.0f, W2 = 0.0f, W3 = 0.0f;
		CubicDecayWeights(W0, W1, W2, W3, TimeSinceTransition, BlendTime);

		InOutAngle += W0 * InAngleOffset + W1 * InAngularVelocityOffset;
		InOutAngularVelocity += W2 * InAngleOffset + W3 * InAngularVelocityOffset;
	}

	/** Specialization of CubicInertializeTransition for angles. Stores velocities in rad/s.
	 *
	 * @param InOutValueOffset		Offset value to update
	 * @param InOutVelocityOffset	Offset velocity to update
	 * @param InSrcValue			Value of the source (value transitioning from)
	 * @param InSrcVelocity			Velocity of the source (velocity transitioning from)
	 * @param InDstValue			Value of the destination (value transitioning to)
	 * @param InDstVelocity			Velocity of the destination (velocity transitioning to)
	 * @param TimeSinceTransition	Time since the last transition
	 * @param BlendTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	static void CubicInertializeTransitionAngle(
		float& InOutAngleOffset,
		float& InOutAngularVelocityOffset,
		float InSrcAngle,
		float InSrcAngularVelocity,
		float InDstAngle,
		float InDstAngularVelocity,
		float TimeSinceTransition,
		float BlendTime)
	{
		float W0 = 0.0f, W1 = 0.0f, W2 = 0.0f, W3 = 0.0f;
		CubicDecayWeights(W0, W1, W2, W3, TimeSinceTransition, BlendTime);

		float Value = InSrcAngle + W0 * InOutAngleOffset + W1 * InOutAngularVelocityOffset;
		float Velocity = InSrcAngularVelocity + W2 * InOutAngleOffset + W3 * InOutAngularVelocityOffset;

		InOutAngleOffset = FMath::FindDeltaAngleRadians(InDstAngle, Value);
		InOutAngularVelocityOffset = Velocity - InDstAngularVelocity;
	}

	/** Specialization of CubicInertializeApply for scales.
	 *
	 * @param InOutValue			The value to apply the inertialization to
	 * @param InOutVelocity			The velocity to apply the inertialization to
	 * @param InValueOffset			Offset value at the time of transition
	 * @param InVelocityOffset		Offset velocity at the time of transition
	 * @param TimeSinceTransition	Time since the last transition
	 * @param BlendTime				Blend over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	static void CubicInertializeApplyScale(
		FVector& InOutScale,
		FVector& InOutScalarVelocity,
		const FVector& InScaleOffset,
		const FVector& InScalarVelocityOffset,
		float TimeSinceTransition,
		float BlendTime)
	{
		float W0 = 0.0f, W1 = 0.0f, W2 = 0.0f, W3 = 0.0f;
		CubicDecayWeights(W0, W1, W2, W3, TimeSinceTransition, BlendTime);

		InOutScale = ScaleExp(W0 * ScaleLog(InScaleOffset) + W1 * InScalarVelocityOffset) * InOutScale;
		InOutScalarVelocity += W2 * ScaleLog(InScaleOffset) + W3 * InScalarVelocityOffset;
	}

	/** Specialization of CubicInertializeTransition for scales.
	 *
	 * @param InOutValueOffset		Offset value to update
	 * @param InOutVelocityOffset	Offset velocity to update
	 * @param InSrcValue			Value of the source (value transitioning from)
	 * @param InSrcVelocity			Velocity of the source (velocity transitioning from)
	 * @param InDstValue			Value of the destination (value transitioning to)
	 * @param InDstVelocity			Velocity of the destination (velocity transitioning to)
	 * @param TimeSinceTransition	Time since the last transition
	 * @param BlendTime				Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 */
	static void CubicInertializeTransitionScale(
		FVector& InOutScaleOffset,
		FVector& InOutScalarVelocityOffset,
		const FVector& InSrcScale,
		const FVector& InSrcScalarVelocity,
		const FVector& InDstScale,
		const FVector& InDstScalarVelocity,
		float TimeSinceTransition,
		float BlendTime)
	{
		float W0 = 0.0f, W1 = 0.0f, W2 = 0.0f, W3 = 0.0f;
		CubicDecayWeights(W0, W1, W2, W3, TimeSinceTransition, BlendTime);

		FVector Value = ScaleExp(W0 * ScaleLog(InOutScaleOffset) + W1 * InOutScalarVelocityOffset) * InSrcScale;
		FVector Velocity = InSrcScalarVelocity + W2 * ScaleLog(InOutScaleOffset) + W3 * InOutScalarVelocityOffset;

		InOutScaleOffset = ScaleDivMax(Value, InDstScale);
		InOutScalarVelocityOffset = Velocity - InDstScalarVelocity;
	}

	/** Extrapolates a value and velocity in place assuming some decay rate on the velocity.
	 *
	 * @param InOutValue			Value to extrapolate from
	 * @param InOutVelocity			Velocity to extrapolate with
	 * @param Time					Amount of time to extrapolate forward
	 * @param SmoothingTime			Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	 */
	template <typename TValue>
	static inline void DeadBlendExtrapolate(
		TValue& InOutValue,
		TValue& InOutVelocity,
		const float Time,
		const float SmoothingTime)
	{
		float NonZeroSmoothingTime = FMath::Max(SmoothingTime, UE_SMALL_NUMBER);
		float Decay = FMath::InvExpApprox(Time / NonZeroSmoothingTime);
		InOutValue = InOutValue + ((InOutVelocity * NonZeroSmoothingTime) * (1.0f - Decay));
		InOutVelocity = InOutVelocity * Decay;
	}

	/** Specialization of DeadBlendExtrapolate for quaternions. Angular velocities are stored in rad/s.
	 *
	 * @param InOutValue			Value to extrapolate from
	 * @param InOutVelocity			Velocity to extrapolate with
	 * @param Time					Amount of time to extrapolate forward
	 * @param SmoothingTime			Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	 */
	static inline void DeadBlendExtrapolateQuat(
		FQuat& InOutValue,
		FVector& InOutVelocity,
		const float Time,
		const float SmoothingTime)
	{
		float NonZeroSmoothingTime = FMath::Max(SmoothingTime, UE_SMALL_NUMBER);
		float Decay = FMath::InvExpApprox(Time / NonZeroSmoothingTime);
		InOutValue = InOutValue * FQuat::MakeFromRotationVector((InOutVelocity * NonZeroSmoothingTime) * (1.0f - Decay));
		InOutVelocity = InOutVelocity * Decay;
	}

	/** Specialization of DeadBlendExtrapolate for scales.
	 *
	 * @param InOutValue			Value to extrapolate from
	 * @param InOutVelocity			Velocity to extrapolate with
	 * @param Time					Amount of time to extrapolate forward
	 * @param SmoothingTime			Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	 */
	static inline void DeadBlendExtrapolateScale(
		FVector& InOutValue,
		FVector& InOutVelocity,
		const float Time,
		const float SmoothingTime)
	{
		float NonZeroSmoothingTime = FMath::Max(SmoothingTime, UE_SMALL_NUMBER);
		float Decay = FMath::InvExpApprox(Time / NonZeroSmoothingTime);
		InOutValue = InOutValue * ScaleExp((InOutVelocity * NonZeroSmoothingTime) * (1.0f - Decay));
		InOutVelocity = InOutVelocity * Decay;
	}

	/** Applies a dead blend to the input value (and velocity), updating them in place by blending them with an extrapolation of the value and velocity at the
	 * point of the last transition. 
	 *
	 * @param InOutValue			The value to apply the inertialization to
	 * @param InOutVelocity			The velocity to apply the inertialization to
	 * @param InValueTransition		Value at the time of the last transition
	 * @param InVelocityTransition	Velocity at the time of the last transition
	 * @param TimeSinceTransition	Time since the last transition
	 * @param BlendTime				Timescale over which to blend out the extrapolation. Larger values result in longer blend times. Can be zero.
	 * @param SmoothingTime			Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	 */
	template <typename TValue>
	static void DeadBlendApply(
		TValue& InOutValue,
		TValue& InOutVelocity,
		const TValue& InValueTransition,
		const TValue& InVelocityTransition,
		float TimeSinceTransition,
		float BlendTime,
		float SmoothingTime)
	{
		TValue ExtrapolatedValue = InValueTransition, ExtrapolatedVelocity = InVelocityTransition;
		DeadBlendExtrapolate(ExtrapolatedValue, ExtrapolatedVelocity, TimeSinceTransition, SmoothingTime);

		float NonZeroBlendTime = FMath::Max(BlendTime, UE_SMALL_NUMBER);
		float Alpha = FMath::Clamp(TimeSinceTransition / NonZeroBlendTime, 0.0f, 1.0f);

		InOutValue = FMath::Lerp(ExtrapolatedValue, InOutValue, Alpha);
		InOutVelocity = FMath::Lerp(ExtrapolatedVelocity, InOutVelocity, Alpha) + 
			(TimeSinceTransition < BlendTime) * ((InOutValue - ExtrapolatedValue) / NonZeroBlendTime);
	}

	/** Transitions a dead blend, computing the new value and velocity at the point of transition.
	 *
	 * @param InOutValueTransition		Value at the point of transition
	 * @param InOutVelocityTransition	Velocity at the point of transition
	 * @param InSrcValue				Value of the source (value transitioning from)
	 * @param InSrcVelocity				Velocity of the source (velocity transitioning from)
	 * @param TimeSinceTransition		Time since the last transition
	 * @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 * @param SmoothingTime				Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	 */
	template <typename TValue>
	static void DeadBlendTransition(
		TValue& InOutValueTransition,
		TValue& InOutVelocityTransition,
		const TValue& InSrcValue,
		const TValue& InSrcVelocity,
		float TimeSinceTransition,
		float BlendTime,
		float SmoothingTime)
	{
		TValue ExtrapolatedValue = InOutValueTransition, ExtrapolatedVelocity = InOutVelocityTransition;
		DeadBlendExtrapolate(ExtrapolatedValue, ExtrapolatedVelocity, TimeSinceTransition, SmoothingTime);

		float NonZeroBlendTime = FMath::Max(BlendTime, UE_SMALL_NUMBER);
		float Alpha = FMath::Clamp(TimeSinceTransition / NonZeroBlendTime, 0.0f, 1.0f);

		InOutValueTransition = FMath::Lerp(ExtrapolatedValue, InSrcValue, Alpha);
		InOutVelocityTransition = FMath::Lerp(ExtrapolatedVelocity, InSrcVelocity, Alpha) + 
			(TimeSinceTransition < BlendTime) * ((InSrcValue - ExtrapolatedValue) / NonZeroBlendTime);
	}

	/** Specialization of DeadBlendApply for quaternions. Stores velocities in rad/s.
	 *
	 * @param InOutValue			The value to apply the inertialization to
	 * @param InOutVelocity			The velocity to apply the inertialization to
	 * @param InValueTransition		Value at the time of the last transition
	 * @param InVelocityTransition	Velocity at the time of the last transition
	 * @param TimeSinceTransition	Time since the last transition
	 * @param BlendTime				Timescale over which to blend out the extrapolation. Larger values result in longer blend times. Can be zero.
	 * @param SmoothingTime			Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	 */
	static void DeadBlendApplyQuat(
		FQuat& InOutValue,
		FVector& InOutVelocity,
		const FQuat& InValueTransition,
		const FVector& InVelocityTransition,
		float TimeSinceTransition,
		float BlendTime,
		float SmoothingTime)
	{
		FQuat ExtrapolatedValue = InValueTransition; FVector ExtrapolatedVelocity = InVelocityTransition;
		DeadBlendExtrapolateQuat(ExtrapolatedValue, ExtrapolatedVelocity, TimeSinceTransition, SmoothingTime);

		float NonZeroBlendTime = FMath::Max(BlendTime, UE_SMALL_NUMBER);
		float Alpha = FMath::Clamp(TimeSinceTransition / NonZeroBlendTime, 0.0f, 1.0f);

		FQuat Direction = (InValueTransition * InOutValue.Inverse()).GetShortestArcWith(FQuat::Identity);
		FVector Diff = (InOutValue * ExtrapolatedValue.Inverse()).GetShortestArcWith(Direction).ToRotationVector();
		
		InOutValue = FQuat::MakeFromRotationVector(Diff * Alpha) * ExtrapolatedValue;
		InOutVelocity = FMath::Lerp(ExtrapolatedVelocity, InOutVelocity, Alpha) + 
			(TimeSinceTransition < BlendTime) * (Diff / NonZeroBlendTime);
	}

	/** Specialization of DeadBlendTransition for quaternions. Stores velocities in rad/s.
	 *
	 * @param InOutValueTransition		Value at the point of transition
	 * @param InOutVelocityTransition	Velocity at the point of transition
	 * @param InSrcValue				Value of the source (value transitioning from)
	 * @param InSrcVelocity				Velocity of the source (velocity transitioning from)
	 * @param TimeSinceTransition		Time since the last transition
	 * @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 * @param SmoothingTime				Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	 */
	static void DeadBlendTransitionQuat(
		FQuat& InOutValueTransition,
		FVector& InOutVelocityTransition,
		const FQuat& InSrcValue,
		const FVector& InSrcVelocity,
		float TimeSinceTransition,
		float BlendTime,
		float SmoothingTime)
	{
		FQuat ExtrapolatedValue = InOutValueTransition; FVector ExtrapolatedVelocity = InOutVelocityTransition;
		DeadBlendExtrapolateQuat(ExtrapolatedValue, ExtrapolatedVelocity, TimeSinceTransition, SmoothingTime);

		float NonZeroBlendTime = FMath::Max(BlendTime, UE_SMALL_NUMBER);
		float Alpha = FMath::Clamp(TimeSinceTransition / NonZeroBlendTime, 0.0f, 1.0f);

		FQuat Direction = (InOutValueTransition * InSrcValue.Inverse()).GetShortestArcWith(FQuat::Identity);
		FVector Diff = (InSrcValue * ExtrapolatedValue.Inverse()).GetShortestArcWith(Direction).ToRotationVector();
	
		InOutValueTransition = FQuat::MakeFromRotationVector(Diff * Alpha) * ExtrapolatedValue;
		InOutVelocityTransition = FMath::Lerp(ExtrapolatedVelocity, InSrcVelocity, Alpha) + 
			(TimeSinceTransition < BlendTime) * (Diff / NonZeroBlendTime);
	}

	/** Specialization of DeadBlendApply for angles. Stores velocities in rad/s.
	 *
	 * @param InOutValue			The value to apply the inertialization to
	 * @param InOutVelocity			The velocity to apply the inertialization to
	 * @param InValueTransition		Value at the time of the last transition
	 * @param InVelocityTransition	Velocity at the time of the last transition
	 * @param TimeSinceTransition	Time since the last transition
	 * @param BlendTime				Timescale over which to blend out the extrapolation. Larger values result in longer blend times. Can be zero.
	 * @param SmoothingTime			Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	 */
	static void DeadBlendApplyAngle(
		float& InOutValue,
		float& InOutVelocity,
		const float& InValueTransition,
		const float& InVelocityTransition,
		float TimeSinceTransition,
		float BlendTime,
		float SmoothingTime)
	{
		float ExtrapolatedValue = InValueTransition, ExtrapolatedVelocity = InVelocityTransition;
		DeadBlendExtrapolate(ExtrapolatedValue, ExtrapolatedVelocity, TimeSinceTransition, SmoothingTime);

		float NonZeroBlendTime = FMath::Max(BlendTime, UE_SMALL_NUMBER);
		float Alpha = FMath::Clamp(TimeSinceTransition / NonZeroBlendTime, 0.0f, 1.0f);

		float Direction = FMath::FindDeltaAngleRadians(ExtrapolatedValue, InValueTransition);
		float Diff = AngleGetShortestArcWith(FMath::FindDeltaAngleRadians(ExtrapolatedValue, InOutValue), Direction);

		InOutValue = (Diff * Alpha) + ExtrapolatedValue;
		InOutVelocity = FMath::Lerp(ExtrapolatedVelocity, InOutVelocity, Alpha) + 
			(TimeSinceTransition < BlendTime) * (Diff / NonZeroBlendTime);
	}

	/** Specialization of DeadBlendTransition for angles. Stores velocities in rad/s.
	 *
	 * @param InOutValueTransition		Value at the point of transition
	 * @param InOutVelocityTransition	Velocity at the point of transition
	 * @param InSrcValue				Value of the source (value transitioning from)
	 * @param InSrcVelocity				Velocity of the source (velocity transitioning from)
	 * @param TimeSinceTransition		Time since the last transition
	 * @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 * @param SmoothingTime				Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	 */
	static void DeadBlendTransitionAngle(
		float& InOutValueTransition,
		float& InOutVelocityTransition,
		const float& InSrcValue,
		const float& InSrcVelocity,
		float TimeSinceTransition,
		float BlendTime,
		float SmoothingTime)
	{
		float ExtrapolatedValue = InOutValueTransition, ExtrapolatedVelocity = InOutVelocityTransition;
		DeadBlendExtrapolate(ExtrapolatedValue, ExtrapolatedVelocity, TimeSinceTransition, SmoothingTime);

		float NonZeroBlendTime = FMath::Max(BlendTime, UE_SMALL_NUMBER);
		float Alpha = FMath::Clamp(TimeSinceTransition / NonZeroBlendTime, 0.0f, 1.0f);

		float Direction = FMath::FindDeltaAngleRadians(ExtrapolatedValue, InOutValueTransition);
		float Diff = AngleGetShortestArcWith(FMath::FindDeltaAngleRadians(ExtrapolatedValue, InSrcValue), Direction);

		InOutValueTransition = (Diff * Alpha) + ExtrapolatedValue;
		InOutVelocityTransition = FMath::Lerp(ExtrapolatedVelocity, InSrcVelocity, Alpha) + 
			(TimeSinceTransition < BlendTime) * (Diff / NonZeroBlendTime);
	}

	/** Specialization of DeadBlendApply for scales.
	 *
	 * @param InOutValue			The value to apply the inertialization to
	 * @param InOutVelocity			The velocity to apply the inertialization to
	 * @param InValueTransition		Value at the time of the last transition
	 * @param InVelocityTransition	Velocity at the time of the last transition
	 * @param TimeSinceTransition	Time since the last transition
	 * @param BlendTime				Timescale over which to blend out the extrapolation. Larger values result in longer blend times. Can be zero.
	 * @param SmoothingTime			Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	 */
	static void DeadBlendApplyScale(
		FVector& InOutValue,
		FVector& InOutVelocity,
		const FVector& InValueTransition,
		const FVector& InVelocityTransition,
		float TimeSinceTransition,
		float BlendTime,
		float SmoothingTime)
	{
		FVector ExtrapolatedValue = InValueTransition, ExtrapolatedVelocity = InVelocityTransition;
		DeadBlendExtrapolateScale(ExtrapolatedValue, ExtrapolatedVelocity, TimeSinceTransition, SmoothingTime);

		float NonZeroBlendTime = FMath::Max(BlendTime, UE_SMALL_NUMBER);
		float Alpha = FMath::Clamp(TimeSinceTransition / NonZeroBlendTime, 0.0f, 1.0f);

		InOutValue = ScaleEerp(ExtrapolatedValue, InOutValue, Alpha);
		InOutVelocity = FMath::Lerp(ExtrapolatedVelocity, InOutVelocity, Alpha) + 
			(TimeSinceTransition < BlendTime) * (ScaleLog(ScaleDivMax(InOutValue, ExtrapolatedValue)) / NonZeroBlendTime);
	}

	/** Specialization of DeadBlendTransition for scales.
	 *
	 * @param InOutValueTransition		Value at the point of transition
	 * @param InOutVelocityTransition	Velocity at the point of transition
	 * @param InSrcValue				Value of the source (value transitioning from)
	 * @param InSrcVelocity				Velocity of the source (velocity transitioning from)
	 * @param TimeSinceTransition		Time since the last transition
	 * @param BlendTime					Timescale over which to apply the offset. Larger values result in longer blend times. Can be zero.
	 * @param SmoothingTime				Decay rate of the extrapolated velocity. Smaller values make the velocity decay faster.
	 */
	static void DeadBlendTransitionScale(
		FVector& InOutValueTransition,
		FVector& InOutVelocityTransition,
		const FVector& InSrcValue,
		const FVector& InSrcVelocity,
		float TimeSinceTransition,
		float BlendTime,
		float SmoothingTime)
	{
		FVector ExtrapolatedValue = InOutValueTransition, ExtrapolatedVelocity = InOutVelocityTransition;
		DeadBlendExtrapolateScale(ExtrapolatedValue, ExtrapolatedVelocity, TimeSinceTransition, SmoothingTime);

		float NonZeroBlendTime = FMath::Max(BlendTime, UE_SMALL_NUMBER);
		float Alpha = FMath::Clamp(TimeSinceTransition / NonZeroBlendTime, 0.0f, 1.0f);

		InOutValueTransition = ScaleEerp(ExtrapolatedValue, InSrcValue, Alpha);
		InOutVelocityTransition = FMath::Lerp(ExtrapolatedVelocity, InSrcVelocity, Alpha) + 
			(TimeSinceTransition < BlendTime) * (ScaleLog(ScaleDivMax(InSrcValue, ExtrapolatedValue)) / NonZeroBlendTime);
	}

};
