// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Algo/ForEach.h"
#include "Algo/Transform.h"
#include "Misc/TVariant.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "Templates/OverrideVoidReturnInvoker.h"

namespace FStaticSpatialIndex
{
	struct FSpatialIndexProfile2D
	{
		enum { Is3D = 0 };
		using FReal = FVector2D::FReal;
		using FVector = FVector2D;
		using FIntPoint = FIntVector2;
		using FBox = FBox2D;
	};

	struct FSpatialIndexProfile3D
	{
		enum { Is3D = 1 };
		using FReal = FVector::FReal;
		using FVector = FVector;
		using FIntPoint = FIntVector;
		using FBox = FBox;
	};

	template <typename Profile>
	inline bool FastSphereAABBIntersection(const typename Profile::FVector& InCenter, const typename Profile::FReal InRadiusSquared, const typename Profile::FBox& InBox)
	{
		const typename Profile::FVector ClosestPoint = Profile::FVector::Max(InBox.Min, Profile::FVector::Min(InCenter, InBox.Max));
		return (ClosestPoint - InCenter).SizeSquared() <= InRadiusSquared;
	}

	struct FSphere
	{
		FSphere(const FVector& InCenter, const FVector::FReal InRadius)
			: Center(InCenter)
			, Radius(InRadius)
		{}

		FVector Center;
		FVector::FReal Radius;
	};

	struct FCone : FSphere
	{
		FCone(const FVector& InCenter, const FVector& InAxis, const FVector::FReal InRadius, const FVector::FReal InAngle)
			: FSphere(InCenter, InRadius)
			, Axis(InAxis)
			, Angle(InAngle)
		{}

		FVector Axis;
		FVector::FReal Angle;
	};

	template <typename Profile>
	inline bool FastConeAABBIntersection(const typename Profile::FVector& InCenter, const typename Profile::FReal InRadiusSquared, const typename Profile::FVector& InAxis, const typename Profile::FReal InAngle, const typename Profile::FReal InSinHalfAngle, const typename Profile::FReal InCosHalfAngle, const typename Profile::FBox& InBox)
	{
		if (FastSphereAABBIntersection<Profile>(InCenter, InRadiusSquared, InBox))
		{
			if constexpr (Profile::Is3D)
			{
				const typename Profile::FVector BoxExtent = InBox.GetExtent();
				const typename Profile::FReal BoxExtentSizeSqr = BoxExtent.SizeSquared();
				const typename Profile::FReal BoxExtentSize = FMath::Sqrt(BoxExtentSizeSqr);
				const typename Profile::FVector U = InCenter - (BoxExtentSize / InSinHalfAngle) * InAxis;
			
				typename Profile::FVector D = InBox.GetCenter() - U;
				typename Profile::FReal DSqr = D | D;
				typename Profile::FReal E = InAxis | D;

				if(E > 0.0f && E * E >= DSqr * FMath::Square(InCosHalfAngle))
				{
					D = InBox.GetCenter() - InCenter;
					DSqr = D | D;
					E = -(InAxis | D);
					if(E > 0.0f && E * E >= DSqr * FMath::Square(InSinHalfAngle))
					{
						return DSqr <= BoxExtentSizeSqr;
					}

					return true;
				}

				return false;
			}
			else
			{
				const FVector2D CenterToMinXMinY((FVector2D(InBox.Min.X, InBox.Min.Y) - InCenter).GetSafeNormal());
				const FVector2D CenterToMaxXMinY((FVector2D(InBox.Max.X, InBox.Min.Y) - InCenter).GetSafeNormal());
				const FVector2D CenterToMaxXMaxY((FVector2D(InBox.Max.X, InBox.Max.Y) - InCenter).GetSafeNormal());
				const FVector2D CenterToMinXMaxY((FVector2D(InBox.Min.X, InBox.Max.Y) - InCenter).GetSafeNormal());

				if (InAngle <= 180.0f)
				{
					const typename Profile::FReal SinAngleMinXMinY = FVector2D::CrossProduct(InAxis, CenterToMinXMinY);
					const typename Profile::FReal SinAngleMaxXMinY = FVector2D::CrossProduct(InAxis, CenterToMaxXMinY);
					const typename Profile::FReal SinAngleMaxXMaxY = FVector2D::CrossProduct(InAxis, CenterToMaxXMaxY);
					const typename Profile::FReal SinAngleMinXMaxY = FVector2D::CrossProduct(InAxis, CenterToMinXMaxY);

					if (SinAngleMinXMinY < -InSinHalfAngle && SinAngleMaxXMinY < -InSinHalfAngle && SinAngleMaxXMaxY < -InSinHalfAngle && SinAngleMinXMaxY < -InSinHalfAngle)
					{
						return false;
					}

					if (SinAngleMinXMinY > InSinHalfAngle && SinAngleMaxXMinY > InSinHalfAngle && SinAngleMaxXMaxY > InSinHalfAngle && SinAngleMinXMaxY > InSinHalfAngle)
					{
						return false;
					}

					const typename Profile::FReal CosAngleMinXMinY = FVector2D::DotProduct(InAxis, CenterToMinXMinY);
					const typename Profile::FReal CosAngleMaxXMinY = FVector2D::DotProduct(InAxis, CenterToMaxXMinY);
					const typename Profile::FReal CosAngleMaxXMaxY = FVector2D::DotProduct(InAxis, CenterToMaxXMaxY);
					const typename Profile::FReal CosAngleMinXMaxY = FVector2D::DotProduct(InAxis, CenterToMinXMaxY);

					if (CosAngleMinXMinY < 0 && CosAngleMaxXMinY < 0 && CosAngleMaxXMaxY < 0 && CosAngleMinXMaxY < 0)
					{
						return false;
					}

					return true;
				}
				else
				{
					const typename Profile::FReal CosAngleMinXMinY = FVector2D::DotProduct(InAxis,CenterToMinXMinY);
					const typename Profile::FReal CosAngleMaxXMinY = FVector2D::DotProduct(InAxis,CenterToMaxXMinY);
					const typename Profile::FReal CosAngleMaxXMaxY = FVector2D::DotProduct(InAxis,CenterToMaxXMaxY);
					const typename Profile::FReal CosAngleMinXMaxY = FVector2D::DotProduct(InAxis,CenterToMinXMaxY);

					if (CosAngleMinXMinY >= 0 || CosAngleMaxXMinY >= 0 || CosAngleMaxXMaxY >= 0 || CosAngleMinXMaxY >= 0)
					{
						return true;
					}

					const typename Profile::FReal InvInSinHalfAngle = FMath::Sin((360.0 - InAngle) * 0.5 * UE_PI / 180.0);
					const typename Profile::FReal InvSinAngleMinXMinY = FVector2D::CrossProduct(-InAxis, CenterToMinXMinY);
					const typename Profile::FReal InvSinAngleMaxXMinY = FVector2D::CrossProduct(-InAxis, CenterToMaxXMinY);
					const typename Profile::FReal InvSinAngleMaxXMaxY = FVector2D::CrossProduct(-InAxis, CenterToMaxXMaxY);
					const typename Profile::FReal InvSinAngleMinXMaxY = FVector2D::CrossProduct(-InAxis, CenterToMinXMaxY);

					if (InvSinAngleMinXMinY > -InvInSinHalfAngle && InvSinAngleMaxXMinY > -InvInSinHalfAngle && InvSinAngleMaxXMaxY > -InvInSinHalfAngle && InvSinAngleMinXMaxY > -InvInSinHalfAngle &&
						InvSinAngleMinXMinY < InvInSinHalfAngle && InvSinAngleMaxXMinY < InvInSinHalfAngle && InvSinAngleMaxXMaxY < InvInSinHalfAngle && InvSinAngleMinXMaxY < InvInSinHalfAngle)
					{
						return false;
					}

					return true;
				}
			}
		}

		return false;
	}
}

template <typename Profile>
struct TStaticSpatialIndexDataInterface
{
	virtual ~TStaticSpatialIndexDataInterface() = default;
	virtual int32 GetNumBox() const = 0;
	virtual const typename Profile::FBox& GetBox(uint32 InIndex) const = 0;
	virtual const typename Profile::FBox* GetBoxes(uint32 InIndex, uint32& OutStride) const = 0;
	virtual uint32 GetAllocatedSize() const = 0;
};

template <typename ValueType, typename Profile, class SpatialIndexType, class ElementsSorter>
class TStaticSpatialIndex : public TStaticSpatialIndexDataInterface<Profile>
{
	using FBox = typename Profile::FBox;

public:
	TStaticSpatialIndex()
		: SpatialIndex(*this)
	{}

	void Init(const TArray<TPair<FBox, ValueType>>& InElements)
	{
		Elements = InElements;
		InitSpatialIndex();
	}

	void Init(TArray<TPair<FBox, ValueType>>&& InElements)
	{
		Elements = MoveTemp(InElements);
		InitSpatialIndex();
	}

	template <class Func>
	void ForEachElement(Func InFunc) const
	{
		TOverrideVoidReturnInvoker Invoker(true, InFunc);

		SpatialIndex.ForEachElement([this, &Invoker](uint32 ValueIndex)
		{
			return Invoker(Elements[ValueIndex].Value);
		});
	}

	template <class Func>
	void ForEachIntersectingElement(const FBox& InBox, Func InFunc) const
	{
		TOverrideVoidReturnInvoker Invoker(true, InFunc);

		SpatialIndex.ForEachIntersectingElement(InBox, [this, &Invoker](uint32 ValueIndex)
		{
			return Invoker(Elements[ValueIndex].Value);
		});
	}

	template <class Func>
	void ForEachIntersectingElement(const FStaticSpatialIndex::FSphere& InSphere, Func InFunc) const
	{
		TOverrideVoidReturnInvoker Invoker(true, InFunc);

		SpatialIndex.ForEachIntersectingElement(InSphere, [this, &Invoker](uint32 ValueIndex)
		{
			return Invoker(Elements[ValueIndex].Value);
		});
	}

	template <class Func>
	void ForEachIntersectingElement(const FStaticSpatialIndex::FCone& InCone, Func InFunc) const
	{
		TOverrideVoidReturnInvoker Invoker(true, InFunc);

		SpatialIndex.ForEachIntersectingElement(InCone, [this, &Invoker](uint32 ValueIndex)
		{
			return Invoker(Elements[ValueIndex].Value);
		});
	}

	void AddReferencedObjects(FReferenceCollector& Collector)
	{
		if constexpr (TIsPointerOrObjectPtrToBaseOf<ValueType, UObject>::Value)
		{
			for (TPair<FBox, ValueType>& Element : Elements)
			{
				Collector.AddReferencedObject(Element.Value);
			}
		}
	}

	// TStaticSpatialIndexDataInterface interface
	virtual int32 GetNumBox() const override
	{
		return Elements.Num();
	}

	virtual const FBox& GetBox(uint32 InIndex) const override
	{
		return Elements[InIndex].Key;
	}

	virtual const FBox* GetBoxes(uint32 InIndex, uint32& OutStride) const override
	{
		OutStride = Elements.GetTypeSize();
		return &Elements[InIndex].Key;
	}

	virtual uint32 GetAllocatedSize() const override
	{
		return sizeof(*this) + Elements.GetAllocatedSize() + SpatialIndex.GetAllocatedSize();
	}

private:
	void InitSpatialIndex()
	{
		// Sort elements to maximize cache coherency during queries
		if (ElementsSorter::NeedSort)
		{
			FBox ElementsBox(ForceInit);
			Algo::ForEach(Elements, [&ElementsBox](const TPair<FBox, ValueType>& Element) { ElementsBox += Element.Key; });

			ElementsSorter Sorter;
			Sorter.Init(ElementsBox);
			Elements.Sort([&Sorter](const TPair<FBox, ValueType>& A, const TPair<FBox, ValueType>B) { return Sorter.Sort(A.Key, B.Key); });
		}

		// Initialize spatial index implementation
		SpatialIndex.Init();
	}

	TArray<TPair<FBox, ValueType>> Elements;
	SpatialIndexType SpatialIndex;
};

namespace FStaticSpatialIndex
{
	template <typename Profile>
	class TImpl
	{
	public:
		TImpl(const TStaticSpatialIndexDataInterface<Profile>& InDataInterface)
			: DataInterface(InDataInterface)
		{}

	protected:
		const TStaticSpatialIndexDataInterface<Profile>& DataInterface;
	};

	template <typename Profile>
	class TListImpl : public TImpl<Profile>
	{
		using FVector = typename Profile::FVector;
		using FBox = typename Profile::FBox;

	public:
		TListImpl(const TStaticSpatialIndexDataInterface<Profile>& InDataInterface)
			: TImpl<Profile>(InDataInterface)
		{}

		void Init() {}

		bool ForEachElement(TFunctionRef<bool(uint32 InValueIndex)> InFunc) const
		{
			for (int32 ValueIndex = 0; ValueIndex < this->DataInterface.GetNumBox(); ValueIndex++)
			{
				if (!InFunc(ValueIndex))
				{
					return false;
				}
			}
			return true;
		}

		bool ForEachIntersectingElement(const FBox& InBox, TFunctionRef<bool(uint32 InValueIndex)> InFunc) const
		{
			uint32 BoxStride;
			const FBox* Box = this->DataInterface.GetBoxes(0, BoxStride);
			for (int32 ValueIndex = 0; ValueIndex < this->DataInterface.GetNumBox(); ValueIndex++, *(uint8**)&Box += BoxStride)
			{
				if (Box->Intersect(InBox))
				{
					if (!InFunc(ValueIndex))
					{
						return false;
					}
				}
			}
			return true;

		}

		bool ForEachIntersectingElement(const FStaticSpatialIndex::FSphere& InSphere, TFunctionRef<bool(uint32 InValueIndex)> InFunc) const
		{
			const typename Profile::FReal RadiusSquared = FMath::Square(InSphere.Radius);

			uint32 BoxStride;
			const FBox* Box = this->DataInterface.GetBoxes(0, BoxStride);
			for (int32 ValueIndex = 0; ValueIndex < this->DataInterface.GetNumBox(); ValueIndex++, *(uint8**)&Box += BoxStride)
			{
				if (FastSphereAABBIntersection<Profile>(FVector(InSphere.Center), RadiusSquared, *Box))
				{
					if (!InFunc(ValueIndex))
					{
						return false;
					}
				}
			}
			return true;
		}

		bool ForEachIntersectingElement(const FStaticSpatialIndex::FCone& InCone, TFunctionRef<bool(uint32 InValueIndex)> InFunc) const
		{
			const typename Profile::FReal RadiusSquared = FMath::Square(InCone.Radius);
			const typename Profile::FReal HalfSinAngle = FMath::Sin(InCone.Angle * 0.5 * UE_PI / 180.0);
			const typename Profile::FReal HalfCosAngle = FMath::Cos(InCone.Angle * 0.5 * UE_PI / 180.0);

			uint32 BoxStride;
			const FBox* Box = this->DataInterface.GetBoxes(0, BoxStride);
			for (int32 ValueIndex = 0; ValueIndex < this->DataInterface.GetNumBox(); ValueIndex++, *(uint8**)&Box += BoxStride)
			{
				if (FastConeAABBIntersection<Profile>(FVector(InCone.Center), RadiusSquared, InCone.Axis, InCone.Angle, HalfSinAngle, HalfCosAngle, *Box))
				{
					if (!InFunc(ValueIndex))
					{
						return false;
					}
				}
			}
			return true;
		}

		uint32 GetAllocatedSize() const
		{
			return sizeof(*this);
		}
	};

	template <typename Profile, int32 MaxNumElementsPerNode = 16, int32 MaxNumElementsPerLeaf = 64>
	class TRTreeImpl : public TImpl<Profile>
	{
		using FVector = typename Profile::FVector;
		using FBox = typename Profile::FBox;

	public:
		TRTreeImpl(const TStaticSpatialIndexDataInterface<Profile>& InDataInterface)
			: TImpl<Profile>(InDataInterface)
		{}

		void Init()
		{
			if (this->DataInterface.GetNumBox())
			{
				// Build leaves
				FNode* CurrentNode = nullptr;
				FBox CurrentNodeBox(ForceInit);
				TArray<FNode> Nodes;

				for (int32 ElementIndex = 0; ElementIndex < this->DataInterface.GetNumBox(); ElementIndex++)
				{
					const FBox& Element = this->DataInterface.GetBox(ElementIndex);

					if (!CurrentNode || (CurrentNode->Content.template Get<typename FNode::FLeafType>().Num() >= MaxNumElementsPerLeaf))
					{
						if (CurrentNode)
						{
							CurrentNode->BoxMin = CurrentNodeBox.Min;
							CurrentNode->BoxMax = CurrentNodeBox.Max;
							CurrentNodeBox.Init();
						}

						CurrentNode = &Nodes.Emplace_GetRef();
						CurrentNode->Content.template Emplace<typename FNode::FLeafType>();
					}

					CurrentNodeBox += Element;
					CurrentNode->Content.template Get<typename FNode::FLeafType>().Add(ElementIndex);
				}

				check(CurrentNode);

				CurrentNode->BoxMin = CurrentNodeBox.Min;
				CurrentNode->BoxMax = CurrentNodeBox.Max;
				CurrentNodeBox.Init();

				// Build nodes
				while (Nodes.Num() > 1)
				{
					CurrentNode = nullptr;
					TArray<FNode> TopNodes;

					for (FNode& Node : Nodes)
					{
						if (!CurrentNode || (CurrentNode->Content.template Get<typename FNode::FNodeType>().Num() >= MaxNumElementsPerNode))
						{
							if (CurrentNode)
							{
								CurrentNode->BoxMin = CurrentNodeBox.Min;
								CurrentNode->BoxMax = CurrentNodeBox.Max;
								CurrentNode->Content.template Get<typename FNode::FNodeType>().Shrink();
								CurrentNodeBox.Init();
							}

							CurrentNode = &TopNodes.Emplace_GetRef();
							CurrentNode->Content.template Emplace<typename FNode::FNodeType>();
						}

						CurrentNodeBox += Node.GetBox();
						CurrentNode->Content.template Get<typename FNode::FNodeType>().Add(MoveTemp(Node));
					}

					check(CurrentNode);

					CurrentNode->BoxMin = CurrentNodeBox.Min;
					CurrentNode->BoxMax = CurrentNodeBox.Max;
					CurrentNode->Content.template Get<typename FNode::FNodeType>().Shrink();
					CurrentNodeBox.Init();

					Nodes = MoveTemp(TopNodes);
				}

				check(Nodes.Num() == 1);
				RootNode = MoveTemp(Nodes[0]);
			}
		}

		bool ForEachElement(TFunctionRef<bool(uint32 InValueIndex)> InFunc) const
		{
			return ForEachElementRecursive(&RootNode, InFunc);
		}

		bool ForEachIntersectingElement(const FBox& InBox, TFunctionRef<bool(uint32 InValueIndex)> InFunc) const
		{
			return ForEachIntersectingElementRecursive(&RootNode, InBox, InFunc);
		}

		bool ForEachIntersectingElement(const FStaticSpatialIndex::FSphere& InSphere, TFunctionRef<bool(uint32 InValueIndex)> InFunc) const
		{
			const typename Profile::FReal RadiusSquared = FMath::Square(InSphere.Radius);
			return ForEachIntersectingElementRecursive(&RootNode, FVector(InSphere.Center), RadiusSquared, InFunc);
		}

		bool ForEachIntersectingElement(const FStaticSpatialIndex::FCone& InCone, TFunctionRef<bool(uint32 InValueIndex)> InFunc) const
		{
			const typename Profile::FReal RadiusSquared = FMath::Square(InCone.Radius);
			const typename Profile::FReal HalfSinAngle = FMath::Sin(InCone.Angle * 0.5 * UE_PI / 180.0);
			const typename Profile::FReal HalfCosAngle = FMath::Cos(InCone.Angle * 0.5 * UE_PI / 180.0);
			return ForEachIntersectingElementRecursive(&RootNode, FVector(InCone.Center), RadiusSquared, FVector(InCone.Axis), InCone.Angle, HalfSinAngle, HalfCosAngle, InFunc);
		}

		uint32 GetAllocatedSize() const
		{
			TFunction<uint32(const FNode*, uint32)> GetAllocatedSizeRecursive = [this, &GetAllocatedSizeRecursive](const FNode* Node, uint32 BaseSize)
			{
				uint32 AllocatedSize = BaseSize;

				if (Node->Content.template IsType<typename FNode::FNodeType>())
				{
					AllocatedSize += Node->Content.template Get<typename FNode::FNodeType>().GetAllocatedSize();

					for (auto& ChildNode : Node->Content.template Get<typename FNode::FNodeType>())
					{
						AllocatedSize += GetAllocatedSizeRecursive(&ChildNode, sizeof(FNode));
					}
				}
				else
				{
					AllocatedSize += Node->Content.template Get<typename FNode::FLeafType>().GetAllocatedSize();
				}

				return AllocatedSize;
			};
	
			return GetAllocatedSizeRecursive(&RootNode, sizeof(*this));
		}

	protected:
		struct FNode
		{
			using FNodeType = TArray<FNode>;
			struct FLeafType
			{
				uint32 StartIndex;
				uint32 NumElements;

				inline FLeafType()
					: StartIndex(0)
					, NumElements(0)
				{}

				inline void Add(uint32 InIndex)
				{
					if (!NumElements)
					{
						StartIndex = InIndex;
					}

					check((StartIndex + NumElements) == InIndex);
					NumElements++;					
				}

				inline uint32 Num() const { return NumElements; }
				inline uint32 GetAllocatedSize() const { return sizeof(*this); }

				struct TIterator
				{
					inline TIterator(uint32 InValue) : Value(InValue) {}
					inline uint32 operator++() { return Value++; }
					inline uint32 operator*() const { return Value; }
					inline bool operator!=(const TIterator& Other) const { return Value != Other.Value; }
					uint32 Value;
				};
				
				inline TIterator begin() { return TIterator(StartIndex); }
				inline TIterator begin() const { return TIterator(StartIndex); }
				inline TIterator end() { return TIterator(StartIndex + NumElements); }
				inline TIterator end() const { return TIterator(StartIndex + NumElements); }
			};
			using FVectorType = typename Profile::FVector;
			using FBoxType = typename Profile::FBox;

			inline FBoxType GetBox() const { return FBoxType(BoxMin, BoxMax); }

			FVectorType BoxMin;
			FVectorType BoxMax;

			TVariant<FNodeType, FLeafType> Content;
		};

		bool ForEachElementRecursive(const FNode* InNode, TFunctionRef<bool(uint32 InValueIndex)> InFunc) const
		{
			if (InNode->Content.template IsType<typename FNode::FNodeType>())
			{
				for (auto& ChildNode : InNode->Content.template Get<typename FNode::FNodeType>())
				{
					if (!ForEachElementRecursive(&ChildNode, InFunc))
					{
						return false;
					}
				}
			}
			else
			{
				for (uint32 ValueIndex : InNode->Content.template Get<typename FNode::FLeafType>())
				{
					if (!InFunc(ValueIndex))
					{
						return false;
					}
				}
			}
			return true;

		}

		bool ForEachIntersectingElementRecursive(const FNode* InNode, const FBox& InBox, TFunctionRef<bool(uint32 InValueIndex)> InFunc) const
		{
			if (InNode->Content.template IsType<typename FNode::FNodeType>())
			{
				for (auto& ChildNode : InNode->Content.template Get<typename FNode::FNodeType>())
				{
					if (ChildNode.GetBox().Intersect(InBox))
					{
						if (!ForEachIntersectingElementRecursive(&ChildNode, InBox, InFunc))
						{
							return false;
						}
					}
				}
			}
			else
			{
				uint32 BoxStride;
				const FBox* Box = this->DataInterface.GetBoxes(InNode->Content.template Get<typename FNode::FLeafType>().StartIndex, BoxStride);

				for (uint32 ValueIndex : InNode->Content.template Get<typename FNode::FLeafType>())
				{
					if (Box->Intersect(InBox))
					{
						if (!InFunc(ValueIndex))
						{
							return false;
						}
					}

					*(uint8**)&Box += BoxStride;
				}
			}
			return true;
		}

		bool ForEachIntersectingElementRecursive(const FNode* InNode, const FVector& InSphereCenter, typename Profile::FReal InRadiusSquared, TFunctionRef<bool(uint32 InValueIndex)> InFunc) const
		{
			if (InNode->Content.template IsType<typename FNode::FNodeType>())
			{
				for (auto& ChildNode : InNode->Content.template Get<typename FNode::FNodeType>())
				{
					if (FastSphereAABBIntersection<Profile>(FVector(InSphereCenter), InRadiusSquared, ChildNode.GetBox()))
					{
						if (!ForEachIntersectingElementRecursive(&ChildNode, InSphereCenter, InRadiusSquared, InFunc))
						{
							return false;
						}
					}
				}
			}
			else
			{
				uint32 BoxStride;
				const FBox* Box = this->DataInterface.GetBoxes(InNode->Content.template Get<typename FNode::FLeafType>().StartIndex, BoxStride);

				for (uint32 ValueIndex : InNode->Content.template Get<typename FNode::FLeafType>())
				{
					if (FastSphereAABBIntersection<Profile>(FVector(InSphereCenter), InRadiusSquared, *Box))
					{
						if (!InFunc(ValueIndex))
						{
							return false;
						}
					}

					*(uint8**)&Box += BoxStride;
				}				
			}
			return true;
		}

		bool ForEachIntersectingElementRecursive(const FNode* InNode, const FVector& InConeCenter, typename Profile::FReal InRadiusSquared, const FVector& InAxis, float InAngle, float InSinHalfAngle, float InCosHalfAngle, TFunctionRef<bool(uint32 InValueIndex)> InFunc) const
		{
			if (InNode->Content.template IsType<typename FNode::FNodeType>())
			{
				for (auto& ChildNode : InNode->Content.template Get<typename FNode::FNodeType>())
				{
					if (FastSphereAABBIntersection<Profile>(FVector(InConeCenter), InRadiusSquared, ChildNode.GetBox()))
					{
						if (!ForEachIntersectingElementRecursive(&ChildNode, InConeCenter, InRadiusSquared, InAxis, InAngle, InSinHalfAngle, InCosHalfAngle, InFunc))
						{
							return false;
						}
					}
				}
			}
			else
			{
				uint32 BoxStride;
				const FBox* Box = this->DataInterface.GetBoxes(InNode->Content.template Get<typename FNode::FLeafType>().StartIndex, BoxStride);

				for (uint32 ValueIndex : InNode->Content.template Get<typename FNode::FLeafType>())
				{
					if (FastConeAABBIntersection<Profile>(FVector(InConeCenter), InRadiusSquared, InAxis, InAngle, InSinHalfAngle, InCosHalfAngle, *Box))
					{
						if (!InFunc(ValueIndex))
						{
							return false;
						}
					}

					*(uint8**)&Box += BoxStride;
				}				
			}
			return true;
		}
		FNode RootNode;
	};

	template <typename Profile>
	class TNodeSorterNoSort
	{
	public:
		enum { NeedSort = 0 };
		using FBox = typename Profile::FBox;
		void Init(const FBox& SortBox) {}
		bool Sort(const FBox& A, const FBox& B) { return false; }
	};

	template <typename Profile>
	class TNodeSorterMinX
	{
	public:
		enum { NeedSort = 1 };
		using FBox = typename Profile::FBox;
		void Init(const FBox& SortBox) {}
		bool Sort(const FBox& A, const FBox& B) { return A.Min.X < B.Min.X; }
	};

	template <typename Profile, int32 BucketSize>
	class TNodeSorterMorton
	{
	public:
		enum { NeedSort = 1 };
		using FBox = typename Profile::FBox;
		using FReal = typename Profile::FReal;
		using FIntPoint = typename Profile::FIntPoint;

		void Init(const FBox& SortBox)
		{}
		
		bool Sort(const FBox& A, const FBox& B)
		{
			FIntPoint CellPosA;
			CellPosA.X = int32(A.GetCenter().X / (FReal)BucketSize);
			CellPosA.Y = int32(A.GetCenter().Y / (FReal)BucketSize);
			if constexpr (Profile::Is3D)
			{
				CellPosA.Z = int32(A.GetCenter().Z / (FReal)BucketSize);
			}
			const uint32 MortonCodeA = MortonEncode(CellPosA);

			FIntPoint CellPosB;
			CellPosB.X = int32(B.GetCenter().X / (FReal)BucketSize);
			CellPosB.Y = int32(B.GetCenter().Y / (FReal)BucketSize);
			if constexpr (Profile::Is3D)
			{
				CellPosB.Z = int32(B.GetCenter().Z / (FReal)BucketSize);
			}
			const uint32 MortonCodeB = MortonEncode(CellPosB);

			return MortonCodeA < MortonCodeB;
		}

	private:
		inline uint32 MortonEncode(const FIntPoint& Point)
		{
			if constexpr (Profile::Is3D)
			{
				return FMath::MortonCode3(Point.X) | (FMath::MortonCode3(Point.Y) << 1) | (FMath::MortonCode3(Point.Z) << 2);
			}
			else
			{
				return FMath::MortonCode2(Point.X) | (FMath::MortonCode2(Point.Y) << 1);
			}
		}
	};

	template <typename Profile, int32 BucketSize>
	class TNodeSorterHilbert
	{
	public:
		enum { NeedSort = 1 };
		using FBox = typename Profile::FBox;
		using FReal = typename Profile::FReal;
		using FIntPoint = typename Profile::FIntPoint;

		void Init(const FBox& SortBox)
		{
			const FReal MaxExtent = SortBox.GetExtent().GetMax();
			const uint32 NumBuckets = FMath::CeilToInt32(MaxExtent / (FReal)BucketSize);
			HilbertOrder = 1 + FMath::CeilLogTwo(NumBuckets);
		}

		bool Sort(const FBox& A, const FBox& B)
		{
			const int32 HilbertCodeA = HilbertEncode({ int32(A.GetCenter().X / (FReal)BucketSize), int32(A.GetCenter().Y / (FReal)BucketSize) }, HilbertOrder);
			const int32 HilbertCodeB = HilbertEncode({ int32(B.GetCenter().X / (FReal)BucketSize), int32(B.GetCenter().Y / (FReal)BucketSize) }, HilbertOrder);
			return HilbertCodeA < HilbertCodeB;
		}

	private:
		uint32 HilbertEncode(const FIntVector2& Point, uint32 Order)
		{
			uint32 Result = 0;

			uint32 State = 0;
			for (int32 i = Order - 1; i >= 0; i--)
			{
				uint32 Row = 4 * State | 2 * ((Point.X >> i) & 1) | ((Point.Y >> i) & 1);
				Result = (Result << 2) | ((0x361e9cb4 >> 2 * Row) & 3);
				State = (0x8fe65831 >> 2 * Row) & 3;
			}

			return Result;
		}

		uint32 HilbertOrder;
	};
}

template <typename ValueType, typename NodeSorter, typename Profile>
class TStaticSpatialIndexList : public TStaticSpatialIndex<ValueType, Profile, FStaticSpatialIndex::TListImpl<Profile>, NodeSorter>
{};

template <typename ValueType, typename NodeSorter, typename Profile>
class TStaticSpatialIndexRTree : public TStaticSpatialIndex<ValueType, Profile, FStaticSpatialIndex::TRTreeImpl<Profile>, NodeSorter>
{};
