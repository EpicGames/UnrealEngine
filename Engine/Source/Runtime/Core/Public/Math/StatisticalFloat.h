// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Math/UnrealMathUtility.h"

// Used to measure a distribution
template <typename T>
struct FStatisticalValue
{
public:
	[[nodiscard]] FStatisticalValue()
		: MinValue()
		, MaxValue()
		, Accumulator()
		, NumSamples()
	{
	}

	void AddSample(T Value)
	{
		if (NumSamples == 0)
		{
			MinValue = MaxValue = Value;
		}
		else
		{
			MinValue = FMath::Min(MinValue, Value);
			MaxValue = FMath::Max(MaxValue, Value);
		}
		Accumulator += Value;
		++NumSamples;
	}

	[[nodiscard]] T GetMinValue() const
	{
		return MinValue;
	}

	[[nodiscard]] T GetMaxValue() const
	{
		return MaxValue;
	}

	[[nodiscard]] T GetAvgValue() const
	{
		return (NumSamples > 0) ? (Accumulator / (double)NumSamples) : T();
	}

	[[nodiscard]] int32 GetCount() const
	{
		return NumSamples;
	}

private:
	T MinValue;
	T MaxValue;
	T Accumulator;
	int32 NumSamples;
};

typedef FStatisticalValue<double> FStatisticalFloat;