// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Include the current implementation of a FTransform, depending on the vector processing mode
 */

#include "CoreTypes.h"
#include "HAL/PreprocessorHelpers.h"
#include "Math/MathFwd.h" // IWYU pragma: export
#include "Math/Quat.h"
#include "Math/ScalarRegister.h"
#include "Math/TransformNonVectorized.h"
#include "Math/TransformVectorized.h"
#include "Misc/LargeWorldCoordinates.h"
#include "Misc/LargeWorldCoordinatesSerializer.h"
#include "Serialization/MemoryLayout.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Templates/IsUECoreType.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

UE_DECLARE_LWC_TYPE(Transform, 3);
class FArchive;
class FMemoryImageWriter;
class FMemoryUnfreezeContent;
class FPointerTableBase;
class FSHA1;
template <typename T> struct TCanBulkSerialize;
template <typename T> struct TIsPODType;

template<> struct TIsPODType<FTransform3f> { enum { Value = true }; };
template<> struct TIsPODType<FTransform3d> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FTransform3f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FTransform3d> { enum { Value = true }; };
template<> struct TCanBulkSerialize<FTransform3f> { enum { Value = false }; }; // LWC_TODO: this was never defined before, should it be true or false?
template<> struct TCanBulkSerialize<FTransform3d> { enum { Value = false }; };
DECLARE_INTRINSIC_TYPE_LAYOUT(FTransform3f);
DECLARE_INTRINSIC_TYPE_LAYOUT(FTransform3d);

namespace UE::Math
{
	/**
	 * Creates a hash value from an FTransform.
	 *
	 * @param Transform the transform to create a hash value for
	 * @return The hash value from the components
	 */
	template<typename T>
	inline uint32 GetTypeHash(const TTransform<T>& Transform)
	{
		uint32 Hash = GetTypeHash(Transform.GetTranslation());
		Hash = HashCombineFast(Hash, GetTypeHash(Transform.GetRotation()));
		Hash = HashCombineFast(Hash, GetTypeHash(Transform.GetScale3D()));

		return Hash;
	}
	
}
