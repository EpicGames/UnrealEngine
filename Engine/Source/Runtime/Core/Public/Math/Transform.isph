// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __TRANSFORM_ISPH__
#define __TRANSFORM_ISPH__

#include "Math/Vector.isph"
#include "Math/Quat.isph"
#include "Math/Matrix.isph"

struct FTransform3f
{
	FVector4f Rotation;
	FVector4f Translation;
	FVector4f Scale3D;
};

struct FTransform
{
	FVector4d Rotation;
	FVector4d Translation;
	FVector4d Scale3D;
};

typedef FTransform FTransform3d;

struct FTransform8
{
	FVector8 Rotation;
	FVector8 Translation;
	FVector8 Scale3D;
};

inline uniform FTransform operator*(const uniform FTransform &A, const uniform FReal F)
{
	uniform FTransform Result;

	Result.Rotation = A.Rotation * F;
	Result.Translation = A.Translation * F;
	Result.Scale3D = A.Scale3D * F;

	return Result;
}

inline uniform FTransform SetTransform(const uniform FVector4 &Rotation, const uniform FVector4 &Translation, const uniform FVector4 &Scale3D)
{
	uniform FTransform Result;

	Result.Rotation = Rotation;
	Result.Translation = Translation;
	Result.Scale3D = Scale3D;

	return Result;
}

inline uniform FTransform8 SetTransform8(const uniform FTransform &T)
{
	uniform FTransform8 Result;

	Result.Rotation = SetVector8(T.Rotation);
	Result.Translation = SetVector8(T.Translation);
	Result.Scale3D = SetVector8(T.Scale3D);

	return Result;
}

inline uniform FMatrix ToMatrixWithScale(const uniform FTransform &T)
{
	const uniform FVector4 RotationX2Y2Z2 = VectorAdd(T.Rotation, T.Rotation);	// x2, y2, z2
	const uniform FVector4 RotationXX2YY2ZZ2 = VectorMultiply(RotationX2Y2Z2, T.Rotation);	// xx2, yy2, zz2

	// The diagonal terms of the rotation matrix are:
	//   (1 - (yy2 + zz2)) * scale
	//   (1 - (xx2 + zz2)) * scale
	//   (1 - (xx2 + yy2)) * scale
	const uniform FVector4 yy2_xx2_xx2 = VectorSwizzle(RotationXX2YY2ZZ2, 1, 0, 0, 0);
	const uniform FVector4 zz2_zz2_yy2 = VectorSwizzle(RotationXX2YY2ZZ2, 2, 2, 1, 0);
	const uniform FVector4 DiagonalSum = VectorAdd(yy2_xx2_xx2, zz2_zz2_yy2);
	const uniform FVector4 Diagonals = VectorSubtract(VectorOne, DiagonalSum);
	const uniform FVector4 OutDiagonals = VectorMultiply(Diagonals, T.Scale3D);

	// Grouping the non-diagonal elements in the rotation block by operations:
	//    ((x*y2,y*z2,x*z2) + (w*z2,w*x2,w*y2)) * scale.xyz and
	//    ((x*y2,y*z2,x*z2) - (w*z2,w*x2,w*y2)) * scale.yxz
	// Rearranging so the LHS and RHS are in the same order as for +
	//    ((x*y2,y*z2,x*z2) - (w*z2,w*x2,w*y2)) * scale.yxz

	// RotBase = x*y2, y*z2, x*z2
	// RotOffset = w*z2, w*x2, w*y2
	const uniform FVector4 x_y_x = VectorSwizzle(T.Rotation, 0, 1, 0, 0);
	const uniform FVector4 y2_z2_z2 = VectorSwizzle(RotationX2Y2Z2, 1, 2, 2, 0);
	const uniform FVector4 RotBase = VectorMultiply(x_y_x, y2_z2_z2);

	const uniform FVector4 w_w_w = VectorReplicate(T.Rotation, 3);
	const uniform FVector4 z2_x2_y2 = VectorSwizzle(RotationX2Y2Z2, 2, 0, 1, 0);
	const uniform FVector4 RotOffset = VectorMultiply(w_w_w, z2_x2_y2);

	// Adds = (RotBase + RotOffset)*Scale3D :  (x*y2 + w*z2) * Scale3D.X , (y*z2 + w*x2) * Scale3D.Y, (x*z2 + w*y2) * Scale3D.Z
	// Subtracts = (RotBase - RotOffset)*Scale3DYZX :  (x*y2 - w*z2) * Scale3D.Y , (y*z2 - w*x2) * Scale3D.Z, (x*z2 - w*y2) * Scale3D.X
	const uniform FVector4 Adds = VectorAdd(RotBase, RotOffset);
	const uniform FVector4 OutAdds = VectorMultiply(Adds, T.Scale3D);
	const uniform FVector4 Scale3DYZXW = VectorSwizzle( T.Scale3D, 1, 2, 0, 3);
	const uniform FVector4 Subtracts = VectorSubtract(RotBase, RotOffset);
	const uniform FVector4 OutSubtracts = VectorMultiply(Subtracts , Scale3DYZXW);

	const uniform FVector4 DiagonalsXYZ_W0 = SetVector4(OutDiagonals.V[0], OutDiagonals.V[1], OutDiagonals.V[2], ZERO);

	// OutMatrix.M[0][0] = (1.0f - (yy2 + zz2)) * Scale.X;    // Diagonal.X
	// OutMatrix.M[0][1] = (xy2 + wz2) * Scale.X;             // Adds.X
	// OutMatrix.M[0][2] = (xz2 - wy2) * Scale.X;             // Subtracts.Z
	// OutMatrix.M[0][3] = 0.0f;                              // DiagonalsXYZ_W0.W
	const uniform FVector4 AddX_DC_DiagX_DC = VectorShuffle(OutAdds, DiagonalsXYZ_W0, 0, 0, 0, 0);
	const uniform FVector4 SubZ_DC_DiagW_DC = VectorShuffle(OutSubtracts, DiagonalsXYZ_W0, 2, 0, 3, 0);
	const uniform FVector4 Row0 = VectorShuffle(AddX_DC_DiagX_DC, SubZ_DC_DiagW_DC, 2, 0, 0, 2);

	// OutMatrix.M[1][0] = (xy2 - wz2) * Scale.Y;             // Subtracts.X
	// OutMatrix.M[1][1] = (1.0f - (xx2 + zz2)) * Scale.Y;    // Diagonal.Y
	// OutMatrix.M[1][2] = (yz2 + wx2) * Scale.Y;             // Adds.Y
	// OutMatrix.M[1][3] = 0.0f;                            // DiagonalsXYZ_W0.W
	const uniform FVector4 SubX_DC_DiagY_DC = VectorShuffle(OutSubtracts, DiagonalsXYZ_W0, 0, 0, 1, 0);
	const uniform FVector4 AddY_DC_DiagW_DC = VectorShuffle(OutAdds, DiagonalsXYZ_W0, 1, 0, 3, 0);
	const uniform FVector4 Row1 = VectorShuffle(SubX_DC_DiagY_DC, AddY_DC_DiagW_DC, 0, 2, 0, 2);

	// OutMatrix.M[2][0] = (xz2 + wy2) * Scale.Z;             // Adds.Z
	// OutMatrix.M[2][1] = (yz2 - wx2) * Scale.Z;             // Subtracts.Y
	// OutMatrix.M[2][2] = (1.0f - (xx2 + yy2)) * Scale.Z;    // Diagonals.Z
	// OutMatrix.M[2][3] = 0.0f;                              // DiagonalsXYZ_W0.W
	const uniform FVector4 AddZ_DC_SubY_DC = VectorShuffle(OutAdds, OutSubtracts, 2, 0, 1, 0);
	const uniform FVector4 Row2 = VectorShuffle(AddZ_DC_SubY_DC, DiagonalsXYZ_W0, 0, 2, 2, 3);

	// OutMatrix.M[3][0] = Translation.X;
	// OutMatrix.M[3][1] = Translation.Y;
	// OutMatrix.M[3][2] = Translation.Z;
	// OutMatrix.M[3][3] = 1.0f;
	const uniform FVector4 Row3 = SetVector4(T.Translation.V[0], T.Translation.V[1], T.Translation.V[2], ONE);

	return SetMatrix(Row0, Row1, Row2, Row3);
}

static inline uniform bool AnyHasNegativeScale(const uniform FVector4 &InScale3D, const uniform FVector4 &InOtherScale3D)
{
	varying FReal S0 = 0;
	varying FReal S1 = 0;
	*((uniform FVector4 *uniform)&S0) = *((uniform FVector4 *uniform)&InScale3D);
	*((uniform FVector4 *uniform)&S1) = *((uniform FVector4 *uniform)&InOtherScale3D);

	return any(S0 < 0 || S1 < 0);
}

static inline void ConstructTransformFromMatrixWithDesiredScale(const uniform FMatrix& AMatrix, const uniform FMatrix& BMatrix, const uniform FVector4& DesiredScale, uniform FTransform& OutTransform)
{
	// the goal of using M is to get the correct orientation
	// but for translation, we still need scale
	uniform FMatrix M = AMatrix * BMatrix;
	MatrixRemoveScaling(M);

	// apply negative scale back to axes
	uniform FVector4 SignedScale = VectorSign(DesiredScale);

	MatrixScaleAxis(M, 0, SignedScale.V[0]);
	MatrixScaleAxis(M, 1, SignedScale.V[1]);
	MatrixScaleAxis(M, 2, SignedScale.V[2]);

	// @note: if you have negative with 0 scale, this will return rotation that is identity
	// since matrix loses that axes
	const uniform FVector4 Rotation = MatrixToQuat(M);
	OutTransform.Rotation = VectorNormalizeQuaternion(Rotation);

	// technically I could calculate this using FTransform but then it does more quat multiplication
	// instead of using Scale in matrix multiplication
	// it's a question of between RemoveScaling vs using FTransform to move translation
	const uniform FVector Translation = MatrixGetOrigin(M);
	OutTransform.Translation = SetVector4(Translation.V[0], Translation.V[1], Translation.V[2], 0);

	OutTransform.Scale3D = DesiredScale;
}

static inline void MultiplyUsingMatrixWithScale(uniform FTransform &OutTransform, const uniform FTransform &A, const uniform FTransform &B)
{
	const uniform FMatrix AMatrix = ToMatrixWithScale(A);
	const uniform FMatrix BMatrix = ToMatrixWithScale(B);
	ConstructTransformFromMatrixWithDesiredScale(AMatrix, BMatrix, VectorMultiply(A.Scale3D, B.Scale3D), OutTransform);
}

/** Returns Multiplied Transform of 2 FTransforms **/
inline void TransformMultiply(uniform FTransform &OutTransform, const uniform FTransform &A, const uniform FTransform &B)
{
	//	When Q = quaternion, S = single scalar scale, and T = translation
	//	QST(A) = Q(A), S(A), T(A), and QST(B) = Q(B), S(B), T(B)

	//	QST (AxB)

	// QST(A) = Q(A)*S(A)*P*-Q(A) + T(A)
	// QST(AxB) = Q(B)*S(B)*QST(A)*-Q(B) + T(B)
	// QST(AxB) = Q(B)*S(B)*[Q(A)*S(A)*P*-Q(A) + T(A)]*-Q(B) + T(B)
	// QST(AxB) = Q(B)*S(B)*Q(A)*S(A)*P*-Q(A)*-Q(B) + Q(B)*S(B)*T(A)*-Q(B) + T(B)
	// QST(AxB) = [Q(B)*Q(A)]*[S(B)*S(A)]*P*-[Q(B)*Q(A)] + Q(B)*S(B)*T(A)*-Q(B) + T(B)

	//	Q(AxB) = Q(B)*Q(A)
	//	S(AxB) = S(A)*S(B)
	//	T(AxB) = Q(B)*S(B)*T(A)*-Q(B) + T(B)

	if (AnyHasNegativeScale(A.Scale3D, B.Scale3D))
	{
		// @note, if you have 0 scale with negative, you're going to lose rotation as it can't convert back to quat
		MultiplyUsingMatrixWithScale(OutTransform, A, B);
	}
	else
	{
		// RotationResult = B.Rotation * A.Rotation
		OutTransform.Rotation = VectorQuaternionMultiply2(B.Rotation, A.Rotation);

		// TranslateResult = B.Rotate(B.Scale * A.Translation) + B.Translate
		const uniform FVector4 ScaledTransA = A.Translation * B.Scale3D;
		const uniform FVector4 RotatedTranslate = VectorQuaternionRotateVector(B.Rotation, ScaledTransA);
		OutTransform.Translation = RotatedTranslate + B.Translation;

		// ScaleResult = Scale.B * Scale.A
		OutTransform.Scale3D = A.Scale3D * B.Scale3D;
	}
}

inline void ScaleTranslation(uniform FTransform &Transform, const uniform FReal Scale)
{
	const uniform FVector4 VectorScale = SetVector4(Scale, Scale, Scale, 0);
	Transform.Translation = VectorMultiply(Transform.Translation, VectorScale);
}

inline uniform FVector4d TransformVector(const uniform FTransform3d &T, const uniform FVector4d& V)
{
	const uniform FVector4d InputVectorW0 = SetVector4(V.V[0], V.V[1], V.V[2], 0);

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const uniform FVector4d ScaledVec = VectorMultiply(T.Scale3D, InputVectorW0);
	return VectorQuaternionRotateVector(T.Rotation, ScaledVec);
}

inline uniform FVector4f TransformVector(const uniform FTransform3f &T, const uniform FVector4f& V)
{
	const uniform FVector4f InputVectorW0 = SetVector4(V.V[0], V.V[1], V.V[2], 0);

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const uniform FVector4f ScaledVec = VectorMultiply(T.Scale3D, InputVectorW0);
	return VectorQuaternionRotateVector(T.Rotation, ScaledVec);
}

inline uniform FVector3d TransformVector(const uniform FTransform3d &T, const uniform FVector3d& V)
{
	const uniform FVector4d InputVectorW0 = SetVector4(V, 0);

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const uniform FVector4d ScaledVec = VectorMultiply(T.Scale3D, InputVectorW0);
	const uniform FVector4d RotatedVec = VectorQuaternionRotateVector(T.Rotation, ScaledVec);
	return SetVector(RotatedVec);
}

inline uniform FVector3f TransformVector(const uniform FTransform3f &T, const uniform FVector3f& V)
{
	const uniform FVector4f InputVectorW0 = SetVector4(V, 0);

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const uniform FVector4f ScaledVec = VectorMultiply(T.Scale3D, InputVectorW0);
	const uniform FVector4f RotatedVec = VectorQuaternionRotateVector(T.Rotation, ScaledVec);
	return SetVector(RotatedVec);
}

inline FVector3d TransformVector(const FTransform3d &T, const FVector3d& V)
{
	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const FVector3d ScaledVec = SetVector(T.Scale3D) * V;
	const FVector3d RotatedVec = VectorQuaternionRotateVector(T.Rotation, ScaledVec);
	return RotatedVec;
}

inline FVector3f TransformVector(const FTransform3f &T, const FVector3f& V)
{
	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const FVector3f ScaledVec = SetVector(T.Scale3D) * V;
	const FVector3f RotatedVec = VectorQuaternionRotateVector(T.Rotation, ScaledVec);
	return RotatedVec;
}

inline FVector3f TransformVector(const uniform FTransform3f &T, const FVector3f& V)
{
	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const FVector3f ScaledVec = SetVector(T.Scale3D) * V;
	const FVector3f RotatedVec = VectorQuaternionRotateVector(T.Rotation, ScaledVec);
	return RotatedVec;
}

inline FVector TransformVector(const uniform FTransform &T, const FVector& V)
{
	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const FVector ScaledVec = SetVector(T.Scale3D) * V;
	const FVector RotatedVec = VectorQuaternionRotateVector(T.Rotation, ScaledVec);
	return RotatedVec;
}

inline uniform FVector TransformVectorNoScale(const uniform FTransform &T, const uniform FVector& V)
{
	const uniform FVector4 InputVectorW0 = SetVector4(V, 0);

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const uniform FVector4 RotatedVec = VectorQuaternionRotateVector(T.Rotation, InputVectorW0);
	return SetVector(RotatedVec);
}

inline FVector3d TransformPosition(const FTransform3d &T, const FVector3d& V)
{
	FVector4d InputVectorW0 = SetVector4(V.V[0], V.V[1], V.V[2], 0);

	//Transform using QST is following
	//QST(P) = Q.Rotate(S*P) + T where Q = quaternion, S = scale, T = translation

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const FVector4d ScaledVec = VectorMultiply(T.Scale3D, InputVectorW0);
	const FVector4d RotatedVec = VectorQuaternionRotateVector(T.Rotation, ScaledVec);

	const FVector4d TranslatedVec = VectorAdd(RotatedVec, T.Translation);

	return SetVector(TranslatedVec);
}

inline FVector3f TransformPosition(const FTransform3f &T, const FVector3f& V)
{
	FVector4f InputVectorW0 = SetVector4(V.V[0], V.V[1], V.V[2], 0);

	//Transform using QST is following
	//QST(P) = Q.Rotate(S*P) + T where Q = quaternion, S = scale, T = translation

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const FVector4f ScaledVec = VectorMultiply(T.Scale3D, InputVectorW0);
	const FVector4f RotatedVec = VectorQuaternionRotateVector(T.Rotation, ScaledVec);

	const FVector4f TranslatedVec = VectorAdd(RotatedVec, T.Translation);

	return SetVector(TranslatedVec);
}

inline FVector TransformPosition(const uniform FTransform &T, const FVector& V)
{
	FVector4 InputVectorW0 = SetVector4(V.V[0], V.V[1], V.V[2], 0);

	//Transform using QST is following
	//QST(P) = Q.Rotate(S*P) + T where Q = quaternion, S = scale, T = translation

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const FVector4 ScaledVec = VectorMultiply(T.Scale3D, InputVectorW0);
	const FVector4 RotatedVec = VectorQuaternionRotateVector(T.Rotation, ScaledVec);

	const FVector4 TranslatedVec = VectorAdd(RotatedVec, (const varying FVector4)T.Translation);

	return SetVector(TranslatedVec);
}

inline FVector3f TransformPosition(const uniform FTransform3f &T, const FVector3f& V)
{
	FVector4f InputVectorW0 = SetVector4(V.V[0], V.V[1], V.V[2], 0);

	//Transform using QST is following
	//QST(P) = Q.Rotate(S*P) + T where Q = quaternion, S = scale, T = translation

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const FVector4f ScaledVec = VectorMultiply(T.Scale3D, InputVectorW0);
	const FVector4f RotatedVec = VectorQuaternionRotateVector(T.Rotation, ScaledVec);

	const FVector4f TranslatedVec = VectorAdd(RotatedVec, (const varying FVector4f)SetVector4(T.Translation.V[0], T.Translation.V[1], T.Translation.V[2], 0.f));

	return SetVector(TranslatedVec);
}

inline uniform FVector3d TransformPosition(const uniform FTransform3d &T, const uniform FVector3d& V)
{
	uniform FVector4d InputVectorW0 = SetVector4(V.V[0], V.V[1], V.V[2], 0);

	//Transform using QST is following
	//QST(P) = Q.Rotate(S*P) + T where Q = quaternion, S = scale, T = translation

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const uniform FVector4d ScaledVec = VectorMultiply(T.Scale3D, InputVectorW0);
	const uniform FVector4d RotatedVec = VectorQuaternionRotateVector(T.Rotation, ScaledVec);

	const uniform FVector4d TranslatedVec = VectorAdd(RotatedVec, T.Translation);

	return SetVector(TranslatedVec);
}

inline uniform FVector3f TransformPosition(const uniform FTransform3f &T, const uniform FVector3f& V)
{
	uniform FVector4f InputVectorW0 = SetVector4(V.V[0], V.V[1], V.V[2], 0);

	//Transform using QST is following
	//QST(P) = Q.Rotate(S*P) + T where Q = quaternion, S = scale, T = translation

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const uniform FVector4f ScaledVec = VectorMultiply(T.Scale3D, InputVectorW0);
	const uniform FVector4f RotatedVec = VectorQuaternionRotateVector(T.Rotation, ScaledVec);

	const uniform FVector4f TranslatedVec = VectorAdd(RotatedVec, T.Translation);

	return SetVector(TranslatedVec);
}

inline static uniform FVector4d TransformPosition(const uniform FTransform3d &T, const uniform FVector4d& V)
{
	//Transform using QST is following
	//QST(P) = Q.Rotate(S*P) + T where Q = quaternion, S = scale, T = translation

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const uniform FVector4d ScaledVec = VectorMultiply(T.Scale3D, V);
	const uniform FVector4d RotatedVec = VectorQuaternionRotateVector(T.Rotation, ScaledVec);

	const uniform FVector4d TranslatedVec = VectorAdd(RotatedVec, T.Translation);

	return TranslatedVec;
}

inline static uniform FVector4f TransformPosition(const uniform FTransform3f &T, const uniform FVector4f& V)
{
	//Transform using QST is following
	//QST(P) = Q.Rotate(S*P) + T where Q = quaternion, S = scale, T = translation

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const uniform FVector4f ScaledVec = VectorMultiply(T.Scale3D, V);
	const uniform FVector4f RotatedVec = VectorQuaternionRotateVector(T.Rotation, ScaledVec);

	const uniform FVector4f TranslatedVec = VectorAdd(RotatedVec, T.Translation);

	return TranslatedVec;
}

inline static uniform FVector8 TransformPosition(const uniform FTransform8 &T, const uniform FVector8& V)
{
	//Transform using QST is following
	//QST(P) = Q.Rotate(S*P) + T where Q = quaternion, S = scale, T = translation

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const uniform FVector8 ScaledVec = VectorMultiply(T.Scale3D, V);
	const uniform FVector8 RotatedVec = VectorQuaternionRotateVector(T.Rotation, ScaledVec);

	const uniform FVector8 TranslatedVec = VectorAdd(RotatedVec, T.Translation);

	return TranslatedVec;
}

inline static FVector TransformPositionNoScale(const uniform FTransform &T, const FVector& V)
{
	FVector4 InputVectorW0 = SetVector4(V.V[0], V.V[1], V.V[2], 0);

	//Transform using QST is following
	//QST(P) = Q.Rotate(S*P) + T where Q = quaternion, S = 1.0f, T = translation

	//RotatedVec = Q.Rotate(V.X, V.Y, V.Z, 0.f)
	const FVector4 RotatedVec = VectorQuaternionRotateVector(T.Rotation, InputVectorW0);
	const FVector4 TranslatedVec = RotatedVec + T.Translation;

	return SetVector(TranslatedVec.V[0], TranslatedVec.V[1], TranslatedVec.V[2]);
}

/** 
 * mathematically if you have 0 scale, it should be infinite, 
 * however, in practice if you have 0 scale, and relative transform doesn't make much sense 
 * anymore because you should be instead of showing gigantic infinite mesh
 * also returning BIG_NUMBER causes sequential NaN issues by multiplying 
 * so we hardcode as 0
 */
inline uniform FVector3d TransformGetSafeScaleReciprocal(const uniform FVector3d& InScale, const uniform double Tolerance)
{
	varying double Scale = 0;
	*((uniform FVector3d *uniform)&Scale) = *((uniform FVector3d *uniform)&InScale);

	const varying double SafeReciprocalScale = abs(Scale) <= Tolerance ? 0 : rcp(Scale);

	return *((uniform FVector3d *uniform)&SafeReciprocalScale);
}

inline uniform FVector3f TransformGetSafeScaleReciprocal(const uniform FVector3f& InScale, const uniform float Tolerance)
{
	varying float Scale = 0;
	*((uniform FVector3f *uniform)&Scale) = *((uniform FVector3f *uniform)&InScale);

	const varying float SafeReciprocalScale = abs(Scale) <= Tolerance ? 0 : rcp(Scale);

	return *((uniform FVector3f *uniform)&SafeReciprocalScale);
}

inline uniform FVector4 TransformGetSafeScaleReciprocal(const uniform FVector4& InScale, const uniform FReal Tolerance)
{
	// SafeReciprocalScale.X = (InScale.X == 0) ? 0.f : 1/InScale.X; // same for YZW

	/// VectorRegister( 1.0f / InScale.x, 1.0f / InScale.y, 1.0f / InScale.z, 1.0f / InScale.w )
	const uniform FVector4 ReciprocalScale = VectorReciprocalAccurate(InScale);

	//VectorRegister( Vec1.x == Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
	const uniform FVector4 ScaleZeroMask = VectorCompareGE(Tolerance, VectorAbs(InScale));

	//const VectorRegister ScaleZeroMask = VectorCompareEQ(InScale, VectorZero());

	// VectorRegister( for each bit i: Mask[i] ? Vec1[i] : Vec2[i] )
	const uniform FVector4 SafeReciprocalScale = VectorSelect(ScaleZeroMask, VectorZero, ReciprocalScale);

	return SafeReciprocalScale;
}

inline FVector InverseTransformPosition(const uniform FTransform &T, const FVector &V)
{
	// (V-Translation)
	FVector TranslatedVec = V - SetVector(T.Translation);

	// ( Rotation.Inverse() * (V-Translation) )
	const FVector VR = VectorQuaternionRotateVector(QuatInverse(T.Rotation), TranslatedVec);

	// GetSafeScaleReciprocal(Scale3D);
	const uniform FVector SafeReciprocal = SetVector(TransformGetSafeScaleReciprocal(T.Scale3D, SMALL_NUMBER));

	// ( Rotation.Inverse() * (V-Translation) ) * GetSafeScaleReciprocal(Scale3D);
	const FVector VResult = VR * SafeReciprocal;

	return VResult;
}

#endif
