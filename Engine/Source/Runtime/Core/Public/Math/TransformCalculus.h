// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector2D.h"
#include "Math/Vector.h"


//////////////////////////////////////////////////////////////////////////
// Transform calculus.
//
// A Transform represents a frame of reference in a local (often orthonormal) coordinate system.
// Essentially a transform represents conversion from a local space A to another local space B.
// Thus, it's commonly written as T[AB]. Keeping this notation of spaces explicit allows
// transformation calculus to be very simple and easily checked using something akin to dimensional analysis:
//
// T[AB] * T[BC] => T[AC]    (aka, Concatenate(T[AB], T[BC])
// T[AB]^-1      => T[BA]    (aka, Inverse(T[AB])
//
// Concatenate is illegal if the LHS output space is not equivalent to the RHS input space:
//
// T[AB] * T[BC]  <--- OK
// T[BA] * T[BC]  <--- illegal; output space is A, input space is B.
//
// So, if you have a spatial hierarchy like so:
//
//                    A
//                 /     \
//                B       C
//               / \     / \
//              D   E   F   G
//                         /
//                        H
//
// You can easily construct the math to move from, say, Space D to Space H using notation
// purely in terms of the forward transforms that define the hierarchy:
//
// T[DH] = T[DB] * T[BA] * T[CA]^-1 * T[GC]^-1 * T[HG]^-1
//
// From a code standpoint, this gives us a well-defined set of methods that can be called on anything that
// can be interpreted as a transform and let it work uniformly. Matrices, Quats, Vectors, Scalars, and other custom classes
// that are unambiguously defined as transformational spaces can be concatenated together with a well-defined meaning.
//
// ------------------------
// Fundamental Operations
// ------------------------
// The fundamental components of the Transform library are a collection of non-member functions (and their overloads):
// * Concatenate(TransformA, TransformB)
// * Inverse(Transform)
// * TransformPoint(Transform, Point)
// * TransformVector(Transform, Vector)
// * TransformCast<Result, Transform>
//
// These operations are NOT member functions to support existing classes in UE4 without modification,
// and to more easily support extending the supported types with other existing classes.
// 
// Concatenate
// -----------
// Generalized transform concatenation. It exists to ensure that any two
// classes that can be thought of as transformational spaces (listed above, along with
// any user-defined classes), can be combined uniformly with well defined semantics.
// For example, given A * B in UE4 means two different things if A and B are matrices or quaternions.
// Concatenate abstracts this notion into an explicit syntax for combining transforms 
// instead of focusing on the mathematical notation.
//
// The result of a concatenate operation depends on the types concatenated, but are guaranteed
// to be another type that supports the fundamental operations. For instance,
// concatenating two translations results in another translation, but concatenating a rotation
// followed by a translation might result in a FMatrix, or potentially another class that efficiently
// contains this transformation (like maybe a class that holds a scalar+quat+vector).
//
// Generally you should not have to worry about the return type. Just Concatenate and then use another
// fundamental operation to transform a position and/or vector through that space. When you need to store the
// result you can either use auto, assume the type, or use TransformCast (covered below) to ensure the type.
//
// In certain rare cases, the result of Concatenate may be ambiguous (FRotator or FQuat?). In such cases,
// there is a Concatenate template overload that allows you to explicitly specify the ResultType. 
// This should rarely be necessary.
//
// Inverse
// -------
// Generalized transform inversion. Takes a transform from space A to space B and returns
// a transform that goes from space B to space A. The output type is guaranteed to be the same.
//
// Transform[Point|Vector]
// --------------------------
// The primary reason to construct a transform from space A to B is to transform things from
// space A to space B. TransformPoint does this for points in space, and TransformVector does
// this for vectors (extents or normals) in space.
//
// There are 2D variants for efficiency. All functions assume a non-projective transform (ie, 
// they don't perform a homogeneous divide). The output type is guaranteed to be the same as the input.
//
// TransformCast<Result, Transform>
// ----------------------------------------------
// The job of the TransformCast is to provide efficient conversion from one transform type
// to another, when possible. This is typically done to store the transform in a specific variable type after
// a sequence of Concatenate and Inverse calls.
// 
// Not all transforms can be cast. For instance, a scale cannot be cast to a translation. Usually
// the output of a cast operation is a generalized transform type like FTransform, FMatrix, etc.
//
// TransformCast supports efficient identity pass-through (Type A -> Type A) so applying a cast in 
// generic code is safe and fast (no copying).
//
// ------------------------------------------------------------------------
// Implementing a custom type.
// ------------------------------------------------------------------------
// While UE4 supports a wide variety of transforms, there is sometimes need to support another custom transform type.
// The core code provides basic scaffolding and a set of recommended practices to make this as easy as 
// possible. This allows most of the functionality to be provided via member functions in the new custom type, while 
// still allowing existing types to be adapted via non-member functions if you choose to do it that way.
// However, you will need to understand a few of the underlying templatized helpers that make this possible:
//   * TransformConverter<T>
//   * ConcatenateRules<T1, T2>
//
// TransformConverter<T>
// ---------------------
// TransformConverter<> implements the meat of TransformCast. It does it's work through the templatized static member 
// function Convert<T2>. By default this method uses a conversion ctor to construct a T from a T2. Therefore your 
// class can either provide such conversion ctors or specialize TransformConverter<T>::Convert<T2> for each transform 
// type you can cast to.
//
// This class is a template struct with a template member to effectively allow partial specialization of either 
// parameter, which function templates do no support. We need to call this as an explicit template call,
// so non-template overloads are not an option. We also need this to support partial specialization for
// the NULL conversion.
//
// ConcatenateRules<T1, T2>
// ------------------------
// In general, the result of a Concatenate call cannot be predicted by the template code. For instance, what 
// is the result of Concatenate(FQuat, FRotator)? What if there are more than one type that can contain the 
// transform (like FMatrix and FTransform)? 
// Concatenate() generally relies on overloads (template or non-template) to do the work. However, requiring 
// all combinations of all types to be overloaded would be quite tedious, and generally unnecessary. Therefore, 
// Concatenate supports a general template form that first converts both parameters to the appropriate return 
// type via TransformCast<>, then calls Concatentate on those:
//
//     return Concatenate(TransformCast<Result>(LHS), TransformCast<Result>(RHS));
//
// This makes it convenient to automatically support flexible concatenation by leveraging the conversion
// mechanics of TransformCast<>.
//
// But how does one determine the "appropriate" return type? This is done via the ConcatenateRules<T1, T2> template,
// which simply defines the appropriate ResultType for Concatenate<T1, T2>:
//
//     typedef XXX ResultType;
//
// The default implementation is empty, meaning there is no support available. There is a partial specialization
// for the same types so the code can always assume that
//
//     Concatenate<T,T> -> T.
//
// Remember that TransformCast<T,T> is a NOOP, so this works efficiently.
// 
// It is up to the implementor of a custom class to define any additional ConcatenateRules for their new type.
//
// Also note that implementing every Concatenate by first upcasting each parameter to the result type may not be very
// efficient. In those cases, providing direct, non-template overload of Concatenate may be better. See the example
// below for details.
//
// ------------------------------------------------------------------------
// Example Custom type
// ------------------------------------------------------------------------
// Say you wanted to create a type that only supports uniform scale followed by 3D translation.
// Let's call this type TranformST for shorthand.
//
// The core code provides default template implementations that pass through to member functions of the custom 
// type. This allows most of the functionality to be centralized in the new custom type, while still allowing
// for non-member overloads if you choose to do it that way.
//
// 
// The following class skeleton provides the basic signature of a class that supports the Transform Calculus:
//
// class FTransformST
// {
// public:
//     explicit FTransformST(float InScale);                        // 1. Used by: TransformConverter
//     explicit FTransformST(const FVector& InTranslation);         // 1. Used by: TransformConverter
//     FMatrix ToMatrix() const;                                    // 2. (OPTIONAL) Used by: TransformConverter<FMatrix>
//     FVector TransformPoint(const FVector& Point) const;         // 3. Used by: TransformPoint
//     FVector TransformVector(const FVector& Vector) const;        // 4. Used by: TransformVector
//     FTransformST Concatenate(const FTransformST& RHS) const;     // 5. Used by: Concatenate
//     FTransformST Inverse() const;                                // 6. Used by: Inverse
// };
//
// inline FTransformST Concatenate(float LHS, const FVector& RHS);  // 7. (OPTIONAL) Used by: Concatenate
// inline FTransformST Concatenate(const FVector& LHS, float RHS);  // 7. (OPTIONAL) Used by: Concatenate
// 
// template<> struct ConcatenateRules<FTransformST, float       > { typedef FTransformST ResultType; };  // 8. (OPTIONAL) Used by: Concatenate
// template<> struct ConcatenateRules<float       , FTransformST> { typedef FTransformST ResultType; };  // 8. (OPTIONAL) Used by: Concatenate
// template<> struct ConcatenateRules<FTransformST, FVector     > { typedef FTransformST ResultType; };  // 8. (OPTIONAL) Used by: Concatenate
// template<> struct ConcatenateRules<FVector     , FTransformST> { typedef FTransformST ResultType; };  // 8. (OPTIONAL) Used by: Concatenate
// 
// template<> struct ConcatenateRules<FTransformST, FMatrix     > { typedef FMatrix ResultType; };  // 9. (OPTIONAL) Used by: Concatenate
// template<> struct ConcatenateRules<FMatrix     , FTransformST> { typedef FMatrix ResultType; };  // 9. (OPTIONAL) Used by: Concatenate
// 
// 1. Provide conversion constructors (can be explicit) to convert a lower level transform into this higher level one.
//    In this case, we can convert any translation or scale to a FTransformST. This will be used by the Concatenate
//    rules below to upcast any lower level types so they can be concatenated together.
// 
// 2. (OPTIONAL) Provide a ToMatrix function to allow this type to be concatenated with FMatrix automatically (which is a common
//    fundamental transform). There is a specialization of TransformConverter for FMatrix that looks for this member function as
//    a convenience to custom class providers.
// 
// 3. Provide a TransformPoint method (and perhaps a 2D version) which will be used by the default template
//    implementation of TransformPoint. If you choose not to provide a member function, you can instead provide
//    a non-template overload of TransformPoint(FTransformTS, FVector).
// 
// 4. Provide a TransformVector method (and perhaps a 2D version) which will be used by the default template
//    implementation of TransformPoint. If you choose not to provide a member function, you can instead provide
//    a non-template overload of TransformVector(FTransformTS, FVector).
// 
// 5. Provide a Concatenate method which will be used by the default template
//    implementation of Concatenate. If you choose not to provide a member function, you can instead provide
//    a non-template overload of Concatenate(FTransformTS, FTransformTS).
// 
// 6. Provide a Inverse method which will be used by the default template
//    implementation of Inverse. If you choose not to provide a member function, you can instead provide
//    a non-template overload of Inverse(FTransformTS).
// 
// 7. Provide some specializations of Concatenate that more efficiently represent the transforms your class supports.
//    In this case, our class can represent an arbitrary combination of uniform scale and translation, so by providing
//    explicit overloads, these more efficient versions will be used instead of promoting both types to FTransformST first.
// 
// 8. Since we don't provide explicit Concatenate combinations for all possibly types (we could), we provide
//    some ConcatenateRules<> to allow the default Concatenate implementation to work with scalars and transform vectors.
// 
// 9. We also provide a set of ConcatenateRules for FMatrix. This ends up using the ToMatrix member function we provided in 2.
// 
//////////////////////////////////////////////////////////////////////////


/** Provides default logic (used by TransformCast) to convert one transform type to another via a conversion ctor. */
template<class TransformType>
struct TransformConverter
{
	/** Efficient NULL conversion. */
	static const TransformType& Convert(const TransformType& Transform)
	{
		return Transform;
	}
	/** 
	 * Default Conversion via a conversion ctor. 
	 * Note we are not using perfect forwarding here. Our types don't generally support move operations, nor do they make sense.
	 * VS 2013 seems to have trouble resolving the specializations below in the presence of perfect forwarding semantics. 
	 */
	template<typename OtherTransformType>
	static TransformType Convert(const OtherTransformType& Transform)
	{
		return TransformType(Transform);
	}
};

/**
 * Casts one TransformType to ResultType using rules laid out by TransformConverter<>::Convert<>(). 
 * 
 * Return type uses decltype to support classes that can return by const-ref more efficiently than returning a new value.
 */
template<typename ResultType, typename TransformType>
[[nodiscard]] inline decltype(auto) TransformCast(const TransformType& Transform)
{
	return TransformConverter<ResultType>::Convert(Transform);
}

/** 
 * Provides default rules defining the result of concatenating two types. By default, nothing is supported
 * because the code cannot know in general what two types result in after concatenation.
 */
template<typename TransformTypeA, typename TransformTypeB>
struct ConcatenateRules
{
};

/** Partial specialization for concatenating two of the same types. Always results in the same type being returned. */
template<typename TransformType>
struct ConcatenateRules<TransformType, TransformType>
{
	typedef TransformType ResultType;
};

/**
 * Concatenates two transforms. Uses TransformCast<> to convert them first.
 * If more efficient means are available to concatenate two transforms, provide a non-template overload (or possibly a specialization).
 * Concatenation is performed in left to right order, so the output space of LHS must match the input space of RHS.
 *
 * @param LHS Transformation that goes from space A to space B
 * @param RHS Transformation that goes from space B to space C.
 * @return a new transform representing the transformation from the input space of LHS to the output space of RHS.
 */
template<typename TransformTypeA, typename TransformTypeB>
[[nodiscard]] inline typename ConcatenateRules<TransformTypeA, TransformTypeB>::ResultType Concatenate(const TransformTypeA& LHS, const TransformTypeB& RHS)
{
	typedef typename ConcatenateRules<TransformTypeA, TransformTypeB>::ResultType ReturnType;
	// If you get a compiler error here about "no member function Concatenate found for TransformType" you know
	// your transform type doesn't support a Concatenate method. Either add one or provide an overload of Concatenate that does this.
	return Concatenate(TransformCast<ReturnType>(LHS), TransformCast<ReturnType>(RHS));
}



/** Special overload that allows one to explicitly define the result type, which applies TransformCast on each argument first. */
template<typename ReturnType, typename LHSType, typename RHSType>
[[nodiscard]] inline ReturnType Concatenate(const LHSType& LHS, const RHSType& RHS)
{
	return Concatenate(TransformCast<ReturnType>(LHS), TransformCast<ReturnType>(RHS));
}

/**
 * Specialization for concatenating two transforms of the same type.
 * By default we try to use a member function on the type.
 * 
 * @param LHS Transformation that goes from space A to space B
 * @param RHS Transformation that goes from space B to space C.
 * @return a new transform representing the transformation from the input space of LHS to the output space of RHS.
 */
template<typename TransformType>
[[nodiscard]] inline auto Concatenate(const TransformType& LHS, const TransformType& RHS) -> decltype(LHS.Concatenate(RHS))
{
	// If you get a compiler error here about "no member function Concatenate found for TransformType" you know
	// your transform type doesn't support a Concatenate method. Either add one or provide an overload of Concatenate that does this.
	return LHS.Concatenate(RHS);
}

/**
 * Concatenates three or more transforms. Uses two-argument Concatenate to do its work,
 * and infers the return type using decltype.
 * 
 * @param TransformAToB Transformation that goes from space A to space B
 * @param TransformBToC Transformation that goes from space B to space C.
 * @param TransformCToRest Transformation that goes from space C onwards.
 * @return a new Transform representing the complete transformation.
 */
template<typename TransformType1, typename TransformType2, typename... TransformTypeRest>
[[nodiscard]] inline decltype(auto) Concatenate(const TransformType1& TransformAToB, const TransformType2& TransformBToC, TransformTypeRest&&... TransformCToRest)
{
	return Concatenate(Concatenate(TransformAToB, TransformBToC), Forward<TransformTypeRest>(TransformCToRest)...);
}

/**
 * Inverts a transform from space A to space B so it transforms from space B to space A.
 * By default attempts to call a member function on the transform type.
 * 
 * @param Transform Input transform from space A to space B.
 * @return Inverted transform from space B to space A.
 */
template<typename TransformType>
[[nodiscard]] inline auto Inverse(const TransformType& Transform) -> decltype(Transform.Inverse())
{
	return Transform.Inverse();
}

/**
 * Generic implementation of TransformPoint. Attempts to use a member function of the TransformType.
 */
template <typename TransformType, typename PositionType>
[[nodiscard]] inline PositionType TransformPoint(const TransformType& Transform, const PositionType& Point)
{
	return Transform.TransformPoint(Point);
} 

/**
 * Generic implementation of TransformVector. Attempts to use a member function of the TransformType.
 */
template <typename TransformType, typename VectorType>
[[nodiscard]] inline VectorType TransformVector(const TransformType& Transform, const VectorType& Vector)
{
	return Transform.TransformVector(Vector);
}

//////////////////////////////////////////////////////////////////////////
// Overloads for uniform Scale.
// 
// This isn't really 2D or 3D specific, but 
// both 2D and 3D leverage uniform scale, and expect these overloads to be available,
// so we go ahead and define them here.
//////////////////////////////////////////////////////////////////////////

/**
 * Specialization for concatenating two scales.
 * 
 * @param LHS Scale that goes from space A to space B
 * @param RHS Scale that goes from space B to space C.
 * @return a new Scale representing the transformation from the input space of LHS to the output space of RHS.
 */
template<typename FloatType UE_REQUIRES(std::is_floating_point_v<FloatType>)>
[[nodiscard]] inline FloatType Concatenate(FloatType LHS, FloatType RHS)
{
	return LHS * RHS;
}

/**
 * Inverts a transform from space A to space B so it transforms from space B to space A.
 * Specialization for uniform scale.
 * 
 * @param Transform Input transform from space A to space B.
 * @return Inverted transform from space B to space A.
 */
template<typename FloatType UE_REQUIRES(std::is_floating_point_v<FloatType>)>
[[nodiscard]] inline FloatType Inverse(FloatType Scale)
{
	return 1.0f / Scale;
}

/**
 * Specialization for uniform Scale.
 */
template<typename PositionType>
[[nodiscard]] inline UE::Math::TVector<PositionType> TransformPoint(float Transform, const UE::Math::TVector<PositionType>& Point)
{
	return Transform * Point;
}

template<typename PositionType>
[[nodiscard]] inline UE::Math::TVector<PositionType> TransformPoint(double Transform, const UE::Math::TVector<PositionType>& Point)
{
	return Transform * Point;
}

/**
 * Specialization for uniform Scale.
 */
template<typename VectorType>
[[nodiscard]] inline UE::Math::TVector<VectorType> TransformVector(float Transform, const UE::Math::TVector<VectorType>& Vector)
{
	return Transform * Vector;
}

template<typename VectorType>
[[nodiscard]] inline UE::Math::TVector<VectorType> TransformVector(double Transform, const UE::Math::TVector<VectorType>& Vector)
{
	return Transform * Vector;
}
