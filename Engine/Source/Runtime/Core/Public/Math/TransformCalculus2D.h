// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector2D.h"
#include "Math/TransformCalculus.h"

#include <type_traits>

//////////////////////////////////////////////////////////////////////////
// Transform calculus for 2D types. UE4 already has a 2D Vector class that we
// will adapt to be interpreted as a translate transform. The rest we create
// new classes for.
//
// The following types are supported
// * float/double    -> represents a uniform scale.
// * FScale2D        -> represents a 2D non-uniform scale.
// * FVector2D       -> represents a 2D translation.
// * FShear2D        -> represents a "2D shear", interpreted as a shear parallel to the X axis followed by a shear parallel to the Y axis.
// * FQuat2D         -> represents a pure 2D rotation.
// * FMatrix2x2      -> represents a general 2D transform.
//
//////////////////////////////////////////////////////////////////////////

template<typename T> class TMatrix2x2;


//////////////////////////////////////////////////////////////////////////
// Adapters for TVector2. 
// 
// Since it is an existing UE4 types, we cannot rely on the default
// template that calls member functions. Instead, we provide direct overloads.
//////////////////////////////////////////////////////////////////////////

namespace UE
{
namespace Math
{

/** Specialization for concatenating two 2D Translations. */
template<typename T>
[[nodiscard]] inline UE::Math::TVector2<T> Concatenate(const UE::Math::TVector2<T>& LHS, const UE::Math::TVector2<T>& RHS)
{
	return LHS + RHS;
}

}	// namespace UE::Math
}	// namespace UE

/** Specialization for inverting a 2D translation. */
template<typename T>
[[nodiscard]] inline UE::Math::TVector2<T> Inverse(const UE::Math::TVector2<T>& Transform)
{
	return -Transform;
}

/** Specialization for TVector2 Translation. */
template<typename T>
[[nodiscard]] inline UE::Math::TVector2<T> TransformPoint(const UE::Math::TVector2<T>& Transform, const UE::Math::TVector2<T>& Point)
{
	return Transform + Point;
}

/** Specialization for FVector2D Translation (does nothing). */
template<typename T>
[[nodiscard]] inline const UE::Math::TVector2<T>& TransformVector(const UE::Math::TVector2<T>& Transform, const UE::Math::TVector2<T>& Vector)
{
	return Vector;
}

//////////////////////////////////////////////////////////////////////////
// Adapters for 2D uniform scale.
// 
// Since it is a fundamental type, we cannot rely on the default
// template that calls member functions. Instead, we provide direct overloads.
//////////////////////////////////////////////////////////////////////////

/**
* Specialization for uniform Scale.
*/
template<typename PositionType>
[[nodiscard]] inline UE::Math::TVector2<PositionType> TransformPoint(float Transform, const UE::Math::TVector2<PositionType>& Point)
{
	return Transform * Point;
}

template<typename PositionType>
[[nodiscard]] inline UE::Math::TVector2<PositionType> TransformPoint(double Transform, const UE::Math::TVector2<PositionType>& Point)
{
	return Transform * Point;
}

/**
* Specialization for uniform Scale.
*/
template<typename VectorType>
[[nodiscard]] inline UE::Math::TVector2<VectorType> TransformVector(float Transform, const UE::Math::TVector2<VectorType>& Vector)
{
	return Transform * Vector;
}

template<typename VectorType>
[[nodiscard]] inline UE::Math::TVector2<VectorType> TransformVector(double Transform, const UE::Math::TVector2<VectorType>& Vector)
{
	return Transform * Vector;
}

/** Represents a 2D non-uniform scale (to disambiguate from an FVector2D, which is used for translation) */
template<typename T>
class TScale2
{
	static_assert(std::is_floating_point_v<T>, "T must be floating point");

public:
	using FReal = T;
	using Vector2Type = UE::Math::TVector2<T>;

	/** Ctor. initialize to an identity scale, 1.0. */
	[[nodiscard]] TScale2() : Scale(1.0f, 1.0f) {}
	/** Ctor. initialize from a uniform scale. */
	[[nodiscard]] explicit TScale2(T InScale) :Scale(InScale, InScale) {}
	/** Ctor. initialize from a non-uniform scale. */
	[[nodiscard]] explicit TScale2(T InScaleX, T InScaleY) :Scale(InScaleX, InScaleY) {}
	/** Ctor. initialize from an FVector defining the 3D scale. */
	template<typename ArgType>
	[[nodiscard]] explicit TScale2(const UE::Math::TVector2<ArgType>& InScale) :Scale(InScale) {}
	
	/** Transform 2D Point */
	template<typename ArgType>
	[[nodiscard]] UE::Math::TVector2<ArgType> TransformPoint(const UE::Math::TVector2<ArgType>& Point) const
	{
		return UE::Math::TVector2<ArgType>(Scale) * Point;
	}

	/** Transform 2D Vector*/
	template<typename ArgType>
	[[nodiscard]] UE::Math::TVector2<ArgType> TransformVector(const UE::Math::TVector2<ArgType>& Vector) const
	{
		return TransformPoint(Vector);
	}

	/** Concatenate two scales. */
	[[nodiscard]] TScale2 Concatenate(const TScale2& RHS) const
	{
		return TScale2(Scale * RHS.Scale);
	}
	/** Invert the scale. */
	[[nodiscard]] TScale2 Inverse() const
	{
		return TScale2(1.0f / Scale.X, 1.0f / Scale.Y);
	}

	/** Equality. */
	[[nodiscard]] bool operator==(const TScale2& Other) const
	{
		return Scale == Other.Scale;
	}
	
	/** Inequality. */
	[[nodiscard]] bool operator!=(const TScale2& Other) const
	{
		return !operator==(Other);
	}

	/** Access to the underlying FVector2D that stores the scale. */
	[[nodiscard]] const Vector2Type& GetVector() const
	{
		return Scale;
	}

private:
	/** Underlying storage of the 2D scale. */
	Vector2Type Scale;
};

/** Base typedefs */
typedef TScale2<float> FScale2f;
typedef TScale2<double> FScale2d;
typedef FScale2f FScale2D; // Default type (for backwards compat)

/** concatenation rules for 2D scales. */
template<typename T> struct ConcatenateRules<float		, TScale2<T>	> { typedef TScale2<T> ResultType; };
template<typename T> struct ConcatenateRules<double		, TScale2<T>	> { typedef TScale2<T> ResultType; };
/** concatenation rules for 2D scales. */
template<typename T> struct ConcatenateRules<TScale2<T>	, float			> { typedef TScale2<T> ResultType; };
template<typename T> struct ConcatenateRules<TScale2<T>	, double		> { typedef TScale2<T> ResultType; };

/** 
 * Represents a 2D shear:
 *   [1 YY]
 *   [XX 1]
 * XX represents a shear parallel to the X axis. YY represents a shear parallel to the Y axis.
 */
template<typename T>
class TShear2
{
	static_assert(std::is_floating_point_v<T>, "T must be floating point");

public:
	using FReal = T;
	using Vector2Type = UE::Math::TVector2<T>;

	/** Ctor. initialize to an identity. */
	TShear2() :Shear(0, 0) {}
	/** Ctor. initialize from a set of shears parallel to the X and Y axis, respectively. */
	explicit TShear2(T ShearX, T ShearY) :Shear(ShearX, ShearY) {}
	/** Ctor. initialize from a 2D vector representing a set of shears parallel to the X and Y axis, respectively. */
	template<typename VType>
	explicit TShear2(const UE::Math::TVector2<VType>& InShear) :Shear((Vector2Type)InShear) {}

	/**
	 * Generates a shear structure based on angles instead of slope.
	 * @param InShearAngles The angles of shear.
	 * @return the sheare structure.
	 */
	template<typename VType>
	[[nodiscard]] static TShear2 FromShearAngles(const UE::Math::TVector2<VType>& InShearAngles)
	{
		// Compute the M (Shear Slot) = CoTan(90 - SlopeAngle)

		// 0 is a special case because Tan(90) == infinity
		T ShearX = InShearAngles.X == 0 ? 0 : (1.0f / FMath::Tan(FMath::DegreesToRadians(90 - FMath::Clamp<T>((T)InShearAngles.X, -89.0f, 89.0f))));
		T ShearY = InShearAngles.Y == 0 ? 0 : (1.0f / FMath::Tan(FMath::DegreesToRadians(90 - FMath::Clamp<T>((T)InShearAngles.Y, -89.0f, 89.0f))));

		return TShear2(ShearX, ShearY);
	}

	/**
	 * Transform 2D Point
	 * [X Y] * [1 YY] == [X+Y*XX Y+X*YY]
	 *         [XX 1]
	 */
	template<typename ArgType>
	[[nodiscard]] UE::Math::TVector2<ArgType> TransformPoint(const UE::Math::TVector2<ArgType>& Point) const
	{
		return Point + UE::Math::TVector2<ArgType>(Point.Y, Point.X) * UE::Math::TVector2<ArgType>(Shear);
	}
	/** Transform 2D Vector*/
	template<typename ArgType>
	[[nodiscard]] UE::Math::TVector2<ArgType> TransformVector(const UE::Math::TVector2<ArgType>& Vector) const
	{
		return TransformPoint(Vector);
	}

	/**
	 * Concatenate two shears. The result is NOT a shear, but must be represented by a generalized 2x2 transform.
	 * Defer the implementation until we can declare a 2x2 matrix.
	 * [1 YYA] * [1 YYB] == [1+YYA*XXB YYB*YYA]
	 * [XXA 1]   [XXB 1]    [XXA+XXB XXA*XXB+1]
	 */
	[[nodiscard]] inline TMatrix2x2<T> Concatenate(const TShear2& RHS) const;

	/**
	 * Invert the shear. The result is NOT a shear, but must be represented by a generalized 2x2 transform.
	 * Defer the implementation until we can declare a 2x2 matrix.
	 * [1 YY]^-1  == 1/(1-YY*XX) * [1 -YY]
	 * [XX 1]                      [-XX 1]
	 */
	[[nodiscard]] TMatrix2x2<T> Inverse() const;


	/** Equality. */
	[[nodiscard]] bool operator==(const TShear2& Other) const
	{
		return Shear == Other.Shear;
	}

	/** Inequality. */
	[[nodiscard]] bool operator!=(const TShear2& Other) const
	{
		return !operator==(Other);
	}

	/** Access to the underlying FVector2D that stores the scale. */
	[[nodiscard]] const Vector2Type& GetVector() const
	{
		return Shear;
	}

private:
	/** Underlying storage of the 2D shear. */
	Vector2Type Shear;
};

/** Base typedefs */
typedef TShear2<float> FShear2f;
typedef TShear2<double> FShear2d;
typedef FShear2f FShear2D;  // Default type (for backwards compat)


/** 
 * Represents a 2D rotation as a complex number (analagous to quaternions). 
 *   Rot(theta) == cos(theta) + i * sin(theta)
 *   General transformation follows complex number algebra from there.
 * Does not use "spinor" notation using theta/2 as we don't need that decomposition for our purposes.
 * This makes the implementation for straightforward and efficient for 2D.
 */
template<typename T>
class TQuat2
{
	static_assert(std::is_floating_point_v<T>, "T must be floating point");

public:
	using FReal = T;
	using Vector2Type = UE::Math::TVector2<T>;

	/** Ctor. initialize to an identity rotation. */
	TQuat2() :Rot(1.0f, 0.0f) {}
	/** Ctor. initialize from a rotation in radians. */
	explicit TQuat2(T RotRadians) :Rot(FMath::Cos(RotRadians), FMath::Sin(RotRadians)) {}
	/** Ctor. initialize from an FVector2D, representing a complex number. */
	template<typename VType>
	explicit TQuat2(const UE::Math::TVector2<VType>& InRot) :Rot((Vector2Type)InRot) {}

	/**
	 * Transform a 2D point by the 2D complex number representing the rotation:
	 * In imaginary land: (x + yi) * (u + vi) == (xu - yv) + (xv + yu)i
	 * 
	 * Looking at this as a matrix, x == cos(A), y == sin(A)
	 * 
	 * [x y] * [ cosA  sinA] == [x y] * [ u v] == [xu-yv xv+yu]
	 *         [-sinA  cosA]            [-v u]
	 *         
	 * Looking at the above results, we see the equivalence with matrix multiplication.
	 */
	template<typename ArgType>
	UE::Math::TVector2<ArgType> TransformPoint(const UE::Math::TVector2<ArgType>& Point) const
	{
		return UE::Math::TVector2<ArgType>(
			Point.X * (ArgType)Rot.X - Point.Y * (ArgType)Rot.Y,
			Point.X * (ArgType)Rot.Y + Point.Y * (ArgType)Rot.X);
	}
	/**
	 * Vector rotation is equivalent to rotating a point.
	 */
	template<typename ArgType>
	UE::Math::TVector2<ArgType> TransformVector(const UE::Math::TVector2<ArgType>& Vector) const
	{
		return TransformPoint(Vector);
	}
	/**
	 * Transform 2 rotations defined by complex numbers:
	 * In imaginary land: (A + Bi) * (C + Di) == (AC - BD) + (AD + BC)i
	 * 
	 * Looking at this as a matrix, A == cos(theta), B == sin(theta), C == cos(sigma), D == sin(sigma):
	 * 
	 * [ A B] * [ C D] == [  AC-BD  AD+BC]
	 * [-B A]   [-D C]    [-(AD+BC) AC-BD]
	 * 
	 * If you look at how the vector multiply works out: [X(AC-BD)+Y(-BC-AD)  X(AD+BC)+Y(-BD+AC)]
	 * you can see it follows the same form of the imaginary form. Indeed, check out how the matrix nicely works
	 * out to [ A B] for a visual proof of the results.
	 *        [-B A]
	 */
	TQuat2 Concatenate(const TQuat2& RHS) const
	{
		return TQuat2(TransformPoint(FVector2D(RHS.Rot)));
	}
	/**
	 * Invert the rotation  defined by complex numbers:
	 * In imaginary land, an inverse is a complex conjugate, which is equivalent to reflecting about the X axis:
	 * Conj(A + Bi) == A - Bi
	 */
	TQuat2 Inverse() const
	{
		return TQuat2(Vector2Type(Rot.X, -Rot.Y));
	}

	/** Equality. */
	bool operator==(const TQuat2& Other) const
	{
		return Rot == Other.Rot;
	}
	
	/** Inequality. */
	bool operator!=(const TQuat2& Other) const
	{
		return !operator==(Other);
	}

	/** Access to the underlying FVector2D that stores the complex number. */
	const Vector2Type& GetVector() const { return Rot; }

private:
	/** Underlying storage of the rotation (X = cos(theta), Y = sin(theta). */
	Vector2Type Rot;
};

/** Base typedefs */
typedef TQuat2<float> FQuat2f;
typedef TQuat2<double> FQuat2d;
typedef FQuat2f FQuat2D; // Default type (for backwards compat)

/**
 * 2x2 generalized matrix. As FMatrix, we assume row vectors, row major storage:
 *    [X Y] * [m00 m01]
 *            [m10 m11]
 */
template<typename T>
class TMatrix2x2
{
	static_assert(std::is_floating_point_v<T>, "T must be floating point");

public:
	using FReal = T;
	using Vector2Type = UE::Math::TVector2<T>;

	/** Ctor. initialize to an identity. */
	TMatrix2x2()
	{
		M[0][0] = 1; M[0][1] = 0;
		M[1][0] = 0; M[1][1] = 1;
	}

	TMatrix2x2(T m00, T m01, T m10, T m11)
	{
		M[0][0] = m00; M[0][1] = m01;
		M[1][0] = m10; M[1][1] = m11;
		DiagnosticCheckNaN();
	}


	/** Ctor. initialize from a scale. */
	explicit TMatrix2x2(T UniformScale)
	{
		M[0][0] = UniformScale; M[0][1] = 0;
		M[1][0] = 0; M[1][1] = UniformScale;
		DiagnosticCheckNaN();
	}

	/** Ctor. initialize from a scale. */
	explicit TMatrix2x2(const TScale2<T>& Scale)
	{
		T ScaleX = (T)Scale.GetVector().X;
		T ScaleY = (T)Scale.GetVector().Y;
		M[0][0] = ScaleX; M[0][1] = 0;
		M[1][0] = 0; M[1][1] = ScaleY;
		DiagnosticCheckNaN();
	}

	/** Factory function. initialize from a 2D shear. */
	explicit TMatrix2x2(const TShear2<T>& Shear)
	{
		T XX = (T)Shear.GetVector().X;
		T YY = (T)Shear.GetVector().Y;
		M[0][0] = 1; M[0][1] =YY;
		M[1][0] = XX; M[1][1] = 1;
		DiagnosticCheckNaN();
	}

	/** Ctor. initialize from a rotation. */
	explicit TMatrix2x2(const TQuat2<T>& Rotation)
	{
		T CosAngle = (T)Rotation.GetVector().X;
		T SinAngle = (T)Rotation.GetVector().Y;
		M[0][0] = CosAngle; M[0][1] = SinAngle;
		M[1][0] = -SinAngle; M[1][1] = CosAngle;
		DiagnosticCheckNaN();
	}

	/**
	 * Transform a 2D point
	 *    [X Y] * [m00 m01]
	 *            [m10 m11]
	 */
	template<typename ArgType>
	UE::Math::TVector2<ArgType> TransformPoint(const UE::Math::TVector2<ArgType>& Point) const
	{
		return UE::Math::TVector2<ArgType>(
			Point.X * (ArgType)M[0][0] + Point.Y * (ArgType)M[1][0],
			Point.X * (ArgType)M[0][1] + Point.Y * (ArgType)M[1][1]);
	}
	/**
	 * Vector transformation is equivalent to point transformation as our matrix is not homogeneous.
	 */
	template<typename ArgType>
	UE::Math::TVector2<ArgType> TransformVector(const UE::Math::TVector2<ArgType>& Vector) const
	{
		return TransformPoint(Vector);
	}
	/**
	 * Concatenate 2 matrices:
	 * [A B] * [E F] == [AE+BG AF+BH]
	 * [C D]   [G H]    [CE+DG CF+DH]
	 */
	TMatrix2x2 Concatenate(const TMatrix2x2& RHS) const
	{
		T A, B, C, D;
		GetMatrix(A, B, C, D);
		T E, F, G, H;
		RHS.GetMatrix(E, F, G, H);
		TMatrix2x2 Result = TMatrix2x2(
			A*E + B*G, A*F + B*H,
			C*E + D*G, C*F + D*H);
		Result.DiagnosticCheckNaN();
		return Result;
	}
	/**
	 * Invert the transform.
	 */
	TMatrix2x2 Inverse() const
	{
		T A, B, C, D;
		GetMatrix(A, B, C, D);
		T InvDet = InverseDeterminant();
		TMatrix2x2 Result = TMatrix2x2(
			 D*InvDet, -B*InvDet,
			-C*InvDet,  A*InvDet);
		Result.DiagnosticCheckNaN();
		return Result;
	}

	/** Equality. */
	bool operator==(const TMatrix2x2& RHS) const
	{
		T A, B, C, D;
		GetMatrix(A, B, C, D);
		T E, F, G, H;
		RHS.GetMatrix(E, F, G, H);
		return
			FMath::IsNearlyEqual(A, E, UE_KINDA_SMALL_NUMBER) &&
			FMath::IsNearlyEqual(B, F, UE_KINDA_SMALL_NUMBER) &&
			FMath::IsNearlyEqual(C, G, UE_KINDA_SMALL_NUMBER) &&
			FMath::IsNearlyEqual(D, H, UE_KINDA_SMALL_NUMBER);
	}

	/** Inequality. */
	bool operator!=(const TMatrix2x2& Other) const
	{
		return !operator==(Other);
	}

	void GetMatrix(float& A, float& B, float& C, float& D) const
	{
		A = (float)M[0][0]; B = (float)M[0][1];
		C = (float)M[1][0]; D = (float)M[1][1];
	}

	void GetMatrix(double& A, double& B, double& C, double& D) const
	{
		A = (double)M[0][0]; B = (double)M[0][1];
		C = (double)M[1][0]; D = (double)M[1][1];
	}

	T Determinant() const
	{
		T A, B, C, D;
		GetMatrix(A, B, C, D);
		return (A*D - B*C);
	}

	T InverseDeterminant() const
	{
		T Det = Determinant();
		checkSlow(Det != 0.0f);
		return 1.0f / Det;
	}

	/** Extracts the squared scale from the matrix (avoids sqrt). */
	TScale2<T> GetScaleSquared() const
	{
		T A, B, C, D;
		GetMatrix(A, B, C, D);
		return TScale2<T>(A*A + B*B, C*C + D*D);
	}

	/** Gets the scale from the matrix. */
	TScale2<T> GetScale() const
	{
		TScale2<T> ScaleSquared = GetScaleSquared();
		return TScale2<T>(FMath::Sqrt(ScaleSquared.GetVector().X), FMath::Sqrt(ScaleSquared.GetVector().Y));
	}

	/** Gets the rotation angle of the matrix. */
	T GetRotationAngle() const
	{
		T A, B, C, D;
		GetMatrix(A, B, C, D);
		return FMath::Atan(C / D);
	}

	/** Determines if the matrix is identity or not. Uses exact float comparison, so rounding error is not considered. */
	bool IsIdentity() const
	{
		return M[0][0] == 1.0f && M[0][1] == 0.0f
			&& M[1][0] == 0.0f && M[1][1] == 1.0f;
	}

	bool IsNearlyIdentity(T ErrorTolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return
			FMath::IsNearlyEqual(M[0][0], 1.0f, ErrorTolerance) &&
			FMath::IsNearlyEqual(M[0][1], 0.0f, ErrorTolerance) &&
			FMath::IsNearlyEqual(M[1][0], 0.0f, ErrorTolerance) &&
			FMath::IsNearlyEqual(M[1][1], 1.0f, ErrorTolerance);
	}

	bool ContainsNaN() const
	{
		return (!FMath::IsFinite(M[0][0]) ||
				!FMath::IsFinite(M[0][1]) ||
				!FMath::IsFinite(M[1][0]) ||
				!FMath::IsFinite(M[1][1]));
	}

#if ENABLE_NAN_DIAGNOSTIC
	inline void DiagnosticCheckNaN() const
	{
		if (ContainsNaN())
		{
			logOrEnsureNanError(TEXT("Matrix2x2 contains NaN"));
		}
	}
#else
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN() const {}
#endif

private:
	T M[2][2];
};

/** Base typedefs */
typedef TMatrix2x2<float> FMatrix2x2f;
typedef TMatrix2x2<double> FMatrix2x2d;
typedef FMatrix2x2f FMatrix2x2; // Default type (for backwards compat)


template<typename T>
inline TMatrix2x2<T> TShear2<T>::Concatenate(const TShear2<T>& RHS) const
{
	T XXA = (T)Shear.X;
	T YYA = (T)Shear.Y;
	T XXB = (T)RHS.Shear.X;
	T YYB = (T)RHS.Shear.Y;
	return TMatrix2x2<T>(
		1+YYA*XXB, YYB*YYA,
		XXA+XXB, XXA*XXB+1);
}

template<typename T>
inline TMatrix2x2<T> TShear2<T>::Inverse() const
{
	T InvDet = 1.0f / T(1.0f - Shear.X*Shear.Y);
	return TMatrix2x2<T>(
		InvDet, T(-Shear.Y * InvDet),
		T(-Shear.X * InvDet), InvDet);
}

/** Concatenation rules for Matrix2x2 and any other type, */
template<typename T> struct ConcatenateRules<float			, TMatrix2x2<T>	> { typedef TMatrix2x2<T> ResultType; };
template<typename T> struct ConcatenateRules<double			, TMatrix2x2<T>	> { typedef TMatrix2x2<T> ResultType; };
template<typename T> struct ConcatenateRules<TScale2<T>		, TMatrix2x2<T>	> { typedef TMatrix2x2<T> ResultType; };
template<typename T> struct ConcatenateRules<TShear2<T>		, TMatrix2x2<T>	> { typedef TMatrix2x2<T> ResultType; };
template<typename T> struct ConcatenateRules<TQuat2<T>		, TMatrix2x2<T>	> { typedef TMatrix2x2<T> ResultType; };
template<typename T> struct ConcatenateRules<TMatrix2x2<T>	, float			> { typedef TMatrix2x2<T> ResultType; };
template<typename T> struct ConcatenateRules<TMatrix2x2<T>	, double		> { typedef TMatrix2x2<T> ResultType; };
template<typename T> struct ConcatenateRules<TMatrix2x2<T>	, TScale2<T>	> { typedef TMatrix2x2<T> ResultType; };
template<typename T> struct ConcatenateRules<TMatrix2x2<T>	, TShear2<T>	> { typedef TMatrix2x2<T> ResultType; };
template<typename T> struct ConcatenateRules<TMatrix2x2<T>	, TQuat2<T>		> { typedef TMatrix2x2<T> ResultType; };

/** Concatenation rules for 2x2 transform types. Convert to 2x2 matrix as the fully decomposed math is not that perf critical right now. */
template<typename T> struct ConcatenateRules<TScale2<T>  , TShear2<T>  > { typedef TMatrix2x2<T> ResultType; };
template<typename T> struct ConcatenateRules<TScale2<T>  , TQuat2<T>   > { typedef TMatrix2x2<T> ResultType; };
template<typename T> struct ConcatenateRules<TShear2<T>  , TScale2<T>  > { typedef TMatrix2x2<T> ResultType; };
template<typename T> struct ConcatenateRules<TQuat2<T>   , TScale2<T>  > { typedef TMatrix2x2<T> ResultType; };
template<typename T> struct ConcatenateRules<TShear2<T>  , TQuat2<T>   > { typedef TMatrix2x2<T> ResultType; };
template<typename T> struct ConcatenateRules<TQuat2<T>   , TShear2<T>  > { typedef TMatrix2x2<T> ResultType; };

 /**
 * Support for generalized 2D affine transforms. 
 * Implemented as a 2x2 transform followed by translation. In matrix form:
 *   [A B 0]
 *   [C D 0]
 *   [X Y 1]
 */
template<typename T>
class TTransform2
{
	static_assert(std::is_floating_point_v<T>, "T must be floating point");

public:
	using FReal = T;
	using Vector2Type = UE::Math::TVector2<T>;
	using Matrix2Type = TMatrix2x2<T>;

	/** Initialize the transform using an identity matrix and a translation. */
	template<typename VType = float>
	TTransform2(const UE::Math::TVector2<VType>& Translation = UE::Math::TVector2<VType>(0.f, 0.f))
		: Trans((Vector2Type)Translation)
	{
	}

	/** Initialize the transform using a uniform scale and a translation. */
	template<typename VType = float>
	explicit TTransform2(T UniformScale, const UE::Math::TVector2<VType>& Translation = UE::Math::TVector2<VType>(0.f, 0.f))
		: M(TScale2<T>(UniformScale)), Trans((Vector2Type)Translation)
	{
	}

	/** Initialize the transform using a 2D scale and a translation. */
	template<typename VType = float>
	explicit TTransform2(const TScale2<T>& Scale, const UE::Math::TVector2<VType>& Translation = UE::Math::TVector2<VType>(0.f, 0.f))
		: M(Scale), Trans((Vector2Type)Translation)
	{
	}

	/** Initialize the transform using a 2D shear and a translation. */
	template<typename VType = float>
	explicit TTransform2(const TShear2<T>& Shear, const UE::Math::TVector2<VType>& Translation = UE::Math::TVector2<VType>(0.f, 0.f))
		: M(Shear), Trans((Vector2Type)Translation)
	{
	}

	/** Initialize the transform using a 2D rotation and a translation. */
	template<typename VType = float>
	explicit TTransform2(const TQuat2<T>& Rot, const UE::Math::TVector2<VType>& Translation = UE::Math::TVector2<VType>(0.f, 0.f))
		: M(Rot), Trans((Vector2Type)Translation)
	{
	}

	/** Initialize the transform using a general 2x2 transform and a translation. */
	template<typename VType = float>
	explicit TTransform2(const TMatrix2x2<T>& Transform, const UE::Math::TVector2<VType>& Translation = UE::Math::TVector2<VType>(0.f, 0.f))
		: M(Transform), Trans((Vector2Type)Translation)
	{
	}

	/**
	 * 2D transformation of a point.  Transforms position, rotation, and scale.
	 */
	template<typename VType>
	UE::Math::TVector2<VType> TransformPoint(const UE::Math::TVector2<VType>& Point) const
	{
		return (UE::Math::TVector2<VType>) ::TransformPoint(Trans, ::TransformPoint(M, (Vector2Type)Point));
	}

	/**
	 * 2D transformation of a vector.  Transforms rotation and scale.
	 */
	template<typename VType>
	UE::Math::TVector2<VType> TransformVector(const UE::Math::TVector2<VType>& Vector) const
	{
		return (UE::Math::TVector2<VType>) ::TransformVector(M, (Vector2Type)Vector);
	}

	/** 
	 * Concatenates two transforms. Result is equivalent to transforming first by this, followed by RHS.
	 *  Concat(A,B) == (P * MA + TA) * MB + TB
	 *              == (P * MA * MB) + TA*MB + TB
	 *  NewM == MA * MB
	 *  NewT == TA * MB + TB
	 */
	TTransform2 Concatenate(const TTransform2& RHS) const
	{
		return TTransform2(
			::Concatenate(M, RHS.M),
			::Concatenate(::TransformPoint(RHS.M, Trans), RHS.Trans));
	}

	/** 
	 * Inverts a transform. So a transform from space A to space B results in a transform from space B to space A. 
	 * Since this class applies the 2x2 transform followed by translation, our inversion logic needs to be able to recast
	 * the result as a M * T. It does it using the following identity:
	 *   (M * T)^-1 == T^-1 * M^-1
	 *   
	 * In homogeneous form, we represent our affine transform like so:
	 *      M    *    T
	 *   [A B 0]   [1 0 0]   [A B 0]
	 *   [C D 0] * [0 1 0] = [C D 0]. This class simply decomposes the 2x2 transform and translation.
	 *   [0 0 1]   [X Y 1]   [X Y 1]
	 * 
	 * But if we were applying the transforms in reverse order (as we need to for the inverse identity above):
	 *    T^-1   *  M^-1
	 *   [1 0 0]   [A B 0]   [A  B  0]  where [X' Y'] = [X Y] * [A B]
	 *   [0 1 0] * [C D 0] = [C  D  0]                          [C D]
	 *   [X Y 1]   [0 0 1]   [X' Y' 1]
	 * 
	 * This can be conceptualized by seeing that a translation effectively defines a new local origin for that 
	 * frame of reference. Since there is a 2x2 transform AFTER that, the concatenated frame of reference has an origin
	 * that is the old origin transformed by the 2x2 transform.
	 * 
	 * In the last equation:
	 * We know that [X Y] is the translation induced by inverting T, or -Translate.
	 * We know that [[A B][C D]] == Inverse(M), so we can represent T^-1 * M^-1 as M'* T' where:
	 *   M' == Inverse(M)
	 *   T' == Inverse(Translate) * Inverse(M)
	 */
	TTransform2 Inverse() const
	{
		Matrix2Type InvM = ::Inverse(M);
		Vector2Type InvTrans = ::TransformPoint(InvM, ::Inverse(Trans));
		return TTransform2(InvM, InvTrans);
	}
	
	/** Equality. */
	bool operator==(const TTransform2& Other) const
	{
		return M == Other.M && Trans == Other.Trans;
	}
	
	/** Inequality. */
	bool operator!=(const TTransform2& Other) const
	{
		return !operator==(Other);
	}

	/** Access to the 2x2 transform */
	const Matrix2Type& GetMatrix() const { return M; }
	/** Access to the translation */
	//const Vector2Type GetTranslation() const { return Vector2Type(Trans); }
	const FVector2D GetTranslation() const { return (FVector2D)Trans; } // TODO: should be Vector2Type in general, but retaining FVector2D for now for compilation backwards compat.

	template<typename VType>
	void SetTranslation(const UE::Math::TVector2<VType>& InTrans) { Trans = (Vector2Type)InTrans; }

	/**
	 * Specialized function to determine if a transform is precisely the identity transform. Uses exact float comparison, so rounding error is not considered.
	 */
	bool IsIdentity() const
	{
		return M.IsIdentity() && Trans == Vector2Type::ZeroVector;
	}

	/**
	 * Converts this affine 2D Transform into an affine 3D transform.
	 */
	UE::Math::TMatrix<T> To3DMatrix() const
	{
		T A, B, C, D;
		M.GetMatrix(A, B, C, D);

		return UE::Math::TMatrix<T>(
			UE::Math::TPlane<T>(      A,      B, 0.0f, 0.0f),
			UE::Math::TPlane<T>(      C,      D, 0.0f, 0.0f),
			UE::Math::TPlane<T>(   0.0f,   0.0f, 1.0f, 0.0f),
			UE::Math::TPlane<T>(Trans.X, Trans.Y, 0.0f, 1.0f)
		);
	}

	/** Utility to check if there are any non-finite values (NaN or Inf) in this transform. */
	bool ContainsNaN() const
	{
		return M.ContainsNaN() || Trans.ContainsNaN();
	}

private:
	Matrix2Type M;
	Vector2Type Trans;
};

/** Core typedefs */
typedef TTransform2<float> FTransform2f;
typedef TTransform2<double> FTransform2d;
typedef FTransform2f FTransform2D; // default type, for backwards compat

template<> struct TIsPODType<FTransform2f> { enum { Value = true }; };
template<> struct TIsPODType<FTransform2d> { enum { Value = true }; };

//////////////////////////////////////////////////////////////////////////
// Concatenate overloads. 
// 
// Efficient overloads for concatenating 2D affine transforms.
// Better than simply upcasting both to FTransform2D first.
//////////////////////////////////////////////////////////////////////////

/** Specialization for concatenating a 2D scale and 2D Translation. */
template<typename T, typename V>
inline TTransform2<T> Concatenate(const TScale2<T>& Scale, const UE::Math::TVector2<V>& Translation)
{
	return TTransform2<T>(Scale, Translation);
}

/** Specialization for concatenating a 2D shear and 2D Translation. */
template<typename T, typename V>
inline TTransform2<T> Concatenate(const TShear2<T>& Shear, const UE::Math::TVector2<V>& Translation)
{
	return TTransform2<T>(Shear, Translation);
}

/** Specialization for concatenating 2D Rotation and 2D Translation. */
template<typename T, typename V>
inline TTransform2<T> Concatenate(const TQuat2<T>& Rot, const UE::Math::TVector2<V>& Translation)
{
	return TTransform2<T>(Rot, Translation);
}

/** Specialization for concatenating 2D generalized transform and 2D Translation. */
template<typename T, typename V>
inline TTransform2<T> Concatenate(const TMatrix2x2<T>& Transform, const UE::Math::TVector2<V>& Translation)
{
	return TTransform2<T>(Transform, Translation);
}

/** Specialization for concatenating transform and 2D Translation. */
template<typename T, typename V>
inline TTransform2<T> Concatenate(const TTransform2<T>& Transform, const UE::Math::TVector2<V>& Translation)
{
	return TTransform2<T>(Transform.GetMatrix(), Concatenate((UE::Math::TVector2<T>)Transform.GetTranslation(), (UE::Math::TVector2<T>)Translation));
}

/** Specialization for concatenating a 2D Translation and 2D scale. */
template<typename T, typename V>
inline TTransform2<T> Concatenate(const UE::Math::TVector2<V>& Translation, const TScale2<T>& Scale)
{
	return TTransform2<T>(Scale, ::TransformPoint(Scale, Translation));
}

/** Specialization for concatenating a 2D Translation and 2D shear. */
template<typename T, typename V>
inline TTransform2<T> Concatenate(const UE::Math::TVector2<V>& Translation, const TShear2<T>& Shear)
{
	return TTransform2<T>(Shear, ::TransformPoint(Shear, Translation));
}

/** Specialization for concatenating 2D Translation and 2D Rotation. */
template<typename T, typename V>
inline TTransform2<T> Concatenate(const UE::Math::TVector2<V>& Translation, const TQuat2<T>& Rot)
{
	return TTransform2<T>(Rot, ::TransformPoint(Rot, Translation));
}

/** Specialization for concatenating 2D Translation and 2D generalized transform. See docs for TTransform2<T>::Inverse for details on how this math is derived. */
template<typename T, typename V>
inline TTransform2<T> Concatenate(const UE::Math::TVector2<V>& Translation, const TMatrix2x2<T>& Transform)
{
	return TTransform2<T>(Transform, ::TransformPoint(Transform, Translation));
}

/** Specialization for concatenating 2D Translation and transform. See docs for TTransform2<T>::Inverse for details on how this math is derived. */
template<typename T, typename V>
inline TTransform2<T> Concatenate(const UE::Math::TVector2<V>& Translation, const TTransform2<T>& Transform)
{
	return TTransform2<T>(Transform.GetMatrix(), Concatenate(::TransformPoint(Transform.GetMatrix(), (UE::Math::TVector2<T>)Translation), (UE::Math::TVector2<T>)Transform.GetTranslation()));
}

/** Helper to determine if a type is based on TTransform2 */
namespace TransformCalculusHelper
{
	template<typename T>
	struct TIsTransform2
	{
		enum { Value = false };
	};

	template<> struct TIsTransform2< FTransform2f > { enum { Value = true }; };
	template<> struct TIsTransform2< FTransform2d > { enum { Value = true }; };
}

/** Partial specialization of ConcatenateRules for FTransform2D and any other type via Upcast to FTransform2D first. Requires a conversion ctor on FTransform2D. Funky template logic so we don't hide the default rule for NULL conversions. */
template<typename TransformType> struct ConcatenateRules<std::enable_if_t<!TransformCalculusHelper::TIsTransform2<TransformType>::Value, FTransform2f>, TransformType> { typedef FTransform2f ResultType; };
template<typename TransformType> struct ConcatenateRules<std::enable_if_t<!TransformCalculusHelper::TIsTransform2<TransformType>::Value, FTransform2d>, TransformType> { typedef FTransform2d ResultType; };
template<typename TransformType> struct ConcatenateRules<TransformType, std::enable_if_t<!TransformCalculusHelper::TIsTransform2<TransformType>::Value, FTransform2f>> { typedef FTransform2f ResultType; };
template<typename TransformType> struct ConcatenateRules<TransformType, std::enable_if_t<!TransformCalculusHelper::TIsTransform2<TransformType>::Value, FTransform2d>> { typedef FTransform2d ResultType; };
