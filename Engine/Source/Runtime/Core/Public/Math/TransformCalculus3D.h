// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector.h"
#include "Math/Rotator.h"
#include "Math/Matrix.h"
#include "Math/RotationMatrix.h"
#include "Math/TranslationMatrix.h"
#include "Math/Quat.h"
#include "Math/ScaleMatrix.h"
#include "Math/TransformCalculus.h"


//////////////////////////////////////////////////////////////////////////
// Transform calculus for 3D types. Since UE4 already has existing 3D transform
// types, this is mostly a set of adapter overloads for the primitive operations
// requires by the transform calculus framework.
//
// The following types are adapted.
// * float/double			-> represents a uniform scale.
// * TScale<T>				-> represents a 3D non-uniform scale.
// * UE::Math::TVector<T>	-> represents a 3D translation.
// * UE::Math::TRotator<T>	-> represents a pure rotation.
// * UE::Math::TQuat<T>		-> represents a pure rotation.
// * UE::Math::TMatrix<T>	-> represents a general 3D homogeneous transform.
//
//////////////////////////////////////////////////////////////////////////

/**
 * Represents a 3D non-uniform scale (to disambiguate from an UE::Math::TVector<T>, which is used for translation).
 * 
 * Serves as a good base example of how to write a class that supports the basic transform calculus
 * operations.
 */
template<typename T>
class TScale
{
	static_assert(std::is_floating_point_v<T>, "T must be floating point");

public:
	using Vector3Type = UE::Math::TVector<T>;

	/** Ctor. initialize to an identity scale, 1.0. */
	[[nodiscard]] TScale() :Scale(1.0f) {}
	/** Ctor. initialize from a uniform scale. */
	[[nodiscard]] explicit TScale(T InScale) :Scale(InScale) {}
	/** Ctor. initialize from an UE::Math::TVector<T> defining the 3D scale. */
	template<typename VType>
	[[nodiscard]] explicit TScale(const UE::Math::TVector<VType>& InScale) : Scale((Vector3Type)InScale) {}
	/** Access to the underlying UE::Math::TVector<T> that stores the scale. */
	[[nodiscard]] const Vector3Type& GetVector() const
	{
		return Scale;
	}
	/** Concatenate two scales. */
	[[nodiscard]] TScale Concatenate(const TScale& RHS) const
	{
		return TScale(Scale * RHS.GetVector());
	}
	/** Invert the scale. */
	[[nodiscard]]  TScale Inverse() const
	{
		return TScale(Vector3Type(1.0f / Scale.X, 1.0f / Scale.Y, 1.0f / Scale.Z));
	}
private:
	/** Underlying storage of the 3D scale. */
	Vector3Type Scale;
};

/** Core typedefs */
typedef TScale<float> FScale3f;
typedef TScale<double> FScale3d;
typedef FScale3f FScale3D; // Default type, for backwards compat

/** Specialization for converting a FMatrix to an FRotator. It uses a non-standard explicit conversion function. */
template<> template<> inline FRotator3f TransformConverter<FRotator3f>::Convert<FMatrix44f>(const FMatrix44f& Transform)
{
	return Transform.Rotator();
}
template<> template<> inline FRotator3d TransformConverter<FRotator3d>::Convert<FMatrix44d>(const FMatrix44d& Transform)
{
	return Transform.Rotator();
}

//////////////////////////////////////////////////////////////////////////
// UE::Math::TMatrix<T> Support
//////////////////////////////////////////////////////////////////////////

/**
 * Converts a generic transform to a matrix using a ToMatrix() member function.
 * Uses decltype to allow some classes to return const-ref types for efficiency.
 * 
 * @param Transform
 * @return the UE::Math::TMatrix<T> stored by the Transform.
 */
template<typename TransformType>
inline auto ToMatrix(const TransformType& Transform) -> decltype(Transform.ToMatrix())
{
	return Transform.ToMatrix();
}

/**
 * Specialization for the NULL Matrix conversion.
 * 
 * @param Scale Uniform Scale
 * @return Matrix that represents the uniform Scale space.
 */
inline const UE::Math::TMatrix<float>& ToMatrix(const UE::Math::TMatrix<float>& Transform)
{
	return Transform;
}

inline const UE::Math::TMatrix<double>& ToMatrix(const UE::Math::TMatrix<double>& Transform)
{
	return Transform;
}

/**
 * Specialization for floats as a uniform scale.
 * 
 * @param Scale Uniform Scale
 * @return Matrix that represents the uniform Scale space.
 */
inline UE::Math::TMatrix<float> ToMatrix(float Scale)
{
	return UE::Math::TScaleMatrix<float>(Scale);
}

inline UE::Math::TMatrix<double> ToMatrix(double Scale)
{
	return UE::Math::TScaleMatrix<double>(Scale);
}

/**
 * Specialization for non-uniform Scale.
 * 
 * @param Scale Non-uniform Scale
 * @return Matrix that represents the non-uniform Scale space.
 */
inline UE::Math::TMatrix<float> ToMatrix(const TScale<float>& Scale)
{
	return UE::Math::TScaleMatrix<float>(UE::Math::TVector<float>(Scale.GetVector()));
}
inline UE::Math::TMatrix<double> ToMatrix(const TScale<double>& Scale)
{
	return UE::Math::TScaleMatrix<double>(UE::Math::TVector<double>(Scale.GetVector()));
}

/**
 * Specialization for translation.
 * 
 * @param Translation Translation
 * @return Matrix that represents the translated space.
 */
inline UE::Math::TMatrix<float> ToMatrix(const UE::Math::TVector<float>& Translation)
{
	return UE::Math::TTranslationMatrix<float>(UE::Math::TVector<float>(Translation));
}
inline UE::Math::TMatrix<double> ToMatrix(const UE::Math::TVector<double>& Translation)
{
	return UE::Math::TTranslationMatrix<double>(UE::Math::TVector<double>(Translation));
}

/**
 * Specialization for rotation.
 * 
 * @param Rotation Rotation
 * @return Matrix that represents the rotated space.
 */
inline UE::Math::TMatrix<float> ToMatrix(const UE::Math::TRotator<float>& Rotation)
{
	return UE::Math::TRotationMatrix<float>(UE::Math::TRotator<float>(Rotation));
}
inline UE::Math::TMatrix<double> ToMatrix(const UE::Math::TRotator<double>& Rotation)
{
	return UE::Math::TRotationMatrix<double>(UE::Math::TRotator<double>(Rotation));
}

/**
 * Specialization for rotation.
 * 
 * @param Rotation Rotation
 * @return Matrix that represents the rotated space.
 */
inline UE::Math::TMatrix<float> ToMatrix(const UE::Math::TQuat<float>& Rotation)
{
	return UE::Math::TRotationMatrix<float>::Make(UE::Math::TQuat<float>(Rotation));
}
inline UE::Math::TMatrix<double> ToMatrix(const UE::Math::TQuat<double>& Rotation)
{
	return UE::Math::TRotationMatrix<double>::Make(UE::Math::TQuat<double>(Rotation));
}

/**
 * Specialization of TransformConverter for UE::Math::TMatrix<T>. Calls ToMatrix() by default.
 * Allows custom types to easily provide support via a ToMatrix() overload or a ToMatrix() member function.
 * Uses decltype to support efficient passthrough of classes that can convert to a UE::Math::TMatrix<T> without creating
 * a new instance.
 */
template<>
struct TransformConverter<FMatrix44f>
{
	template<typename OtherTransformType>
	static auto Convert(const OtherTransformType& Transform) -> decltype((FMatrix44f)ToMatrix(Transform))
	{
		return (FMatrix44f)ToMatrix(Transform);
	}
};

template<>
struct TransformConverter<FMatrix44d>
{
	template<typename OtherTransformType>
	static auto Convert(const OtherTransformType& Transform) -> decltype((FMatrix44d)ToMatrix(Transform))
	{
		return (FMatrix44d)ToMatrix(Transform);
	}
};

/** concatenation rules for basic UE4 types. */
template<typename T> struct ConcatenateRules<float					, TScale<T>				> { typedef TScale<T> ResultType; };
template<typename T> struct ConcatenateRules<double					, TScale<T>				> { typedef TScale<T> ResultType; };
template<typename T> struct ConcatenateRules<TScale<T>				, float					> { typedef TScale<T> ResultType; };
template<typename T> struct ConcatenateRules<TScale<T>				, double				> { typedef TScale<T> ResultType; };
template<typename T> struct ConcatenateRules<float					, UE::Math::TVector<T>	> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<double					, UE::Math::TVector<T>	> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TVector<T>	, float					> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TVector<T>	, double				> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<float					, UE::Math::TRotator<T>	> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<double					, UE::Math::TRotator<T>	> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TRotator<T>	, float					> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TRotator<T>	, double				> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<float					, UE::Math::TQuat<T>	> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<double					, UE::Math::TQuat<T>	> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TQuat<T>		, float					> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TQuat<T>		, double				> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<float					, UE::Math::TMatrix<T>	> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<double					, UE::Math::TMatrix<T>	> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TMatrix<T>	, float					> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TMatrix<T>	, double				> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<TScale<T>				, UE::Math::TVector<T>  > { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TVector<T>	, TScale<T>				> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<TScale<T>				, UE::Math::TRotator<T> > { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TRotator<T>	, TScale<T>				> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<TScale<T>				, UE::Math::TQuat<T>    > { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TQuat<T>		, TScale<T>				> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<TScale<T>				, UE::Math::TMatrix<T>  > { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TMatrix<T>	, TScale<T>				> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TVector<T>	, UE::Math::TRotator<T> > { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TRotator<T>	, UE::Math::TVector<T>  > { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TVector<T>	, UE::Math::TQuat<T>    > { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TQuat<T>		, UE::Math::TVector<T>  > { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TVector<T>	, UE::Math::TMatrix<T>  > { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TMatrix<T>	, UE::Math::TVector<T>  > { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TRotator<T>	, UE::Math::TQuat<T>    > { typedef UE::Math::TQuat<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TQuat<T>		, UE::Math::TRotator<T> > { typedef UE::Math::TQuat<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TRotator<T>	, UE::Math::TMatrix<T>  > { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TMatrix<T>	, UE::Math::TRotator<T> > { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TQuat<T>		, UE::Math::TMatrix<T>  > { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TMatrix<T>	, UE::Math::TQuat<T>    > { typedef UE::Math::TMatrix<T> ResultType; };

//////////////////////////////////////////////////////////////////////////
// Concatenate overloads. 
// 
// Since these are existing UE4 types, we cannot rely on the default
// template that calls member functions. Instead, we provide direct overloads.
//////////////////////////////////////////////////////////////////////////


namespace UE
{
namespace Math
{
	
/**
 * Specialization for concatenating two Matrices.
 * 
 * @param LHS rotation that goes from space A to space B
 * @param RHS rotation that goes from space B to space C.
 * @return a new rotation representing the transformation from the input space of LHS to the output space of RHS.
 */
template<typename T>
inline UE::Math::TMatrix<T> Concatenate(const UE::Math::TMatrix<T>& LHS, const UE::Math::TMatrix<T>& RHS)
{
	return LHS * RHS;
}


/**
* Specialization for concatenating two translations.
* 
* @param LHS Translation that goes from space A to space B
* @param RHS Translation that goes from space B to space C.
* @return a new Translation representing the transformation from the input space of LHS to the output space of RHS.
*/
template<typename T>
inline UE::Math::TVector<T> Concatenate(const UE::Math::TVector<T>& LHS, const UE::Math::TVector<T>& RHS)
{
	return LHS + RHS;
}
	

/**
 * Specialization for concatenating two rotations.
 *
 * NOTE: UE::Math::TQuat<T> concatenates right to left, opposite of how UE::Math::TMatrix<T> implements it.
 *       Confusing, no? That's why we have these high level functions!
 * 
 * @param LHS rotation that goes from space A to space B
 * @param RHS rotation that goes from space B to space C.
 * @return a new rotation representing the transformation from the input space of LHS to the output space of RHS.
 */
template<typename T>
inline UE::Math::TQuat<T> Concatenate(const UE::Math::TQuat<T>& LHS, const UE::Math::TQuat<T>& RHS)
{
	return RHS * LHS;
}

} // namespace UE::Math
} // namespace UE

/**
 * Specialization for concatenating two rotations.
 *
 * @param LHS rotation that goes from space A to space B
 * @param RHS rotation that goes from space B to space C.
 * @return a new rotation representing the transformation from the input space of LHS to the output space of RHS.
 */
template<typename T>
inline UE::Math::TRotator<T> Concatenate(const UE::Math::TRotator<T>& LHS, const UE::Math::TRotator<T>& RHS)
{
	//@todo implement a more efficient way to do this.
	return TransformCast<UE::Math::TRotator<T>>(Concatenate(TransformCast<UE::Math::TMatrix<T>>(LHS), TransformCast<UE::Math::TMatrix<T>>(RHS)));
}


//////////////////////////////////////////////////////////////////////////
// Inverse overloads. 
// 
// Since these are existing UE4 types, we cannot rely on the default
// template that calls member functions. Instead, we provide direct overloads.
//////////////////////////////////////////////////////////////////////////

/**
 * Inverts a transform from space A to space B so it transforms from space B to space A.
 * Specialization for UE::Math::TMatrix<T>.
 * 
 * @param Transform Input transform from space A to space B.
 * @return Inverted transform from space B to space A.
 */
template<typename T>
inline UE::Math::TMatrix<T> Inverse(const UE::Math::TMatrix<T>& Transform)
{
	return Transform.Inverse();
}

/**
 * Inverts a transform from space A to space B so it transforms from space B to space A.
 * Specialization for UE::Math::TRotator<T>.
 * 
 * @param Transform Input transform from space A to space B.
 * @return Inverted transform from space B to space A.
 */
template<typename T>
inline UE::Math::TRotator<T> Inverse(const UE::Math::TRotator<T>& Transform)
{
	UE::Math::TVector<T> EulerAngles = Transform.Euler();
	return UE::Math::TRotator<T>::MakeFromEuler(UE::Math::TVector<T>(-EulerAngles.Z, -EulerAngles.Y, -EulerAngles.X));
}

/**
 * Inverts a transform from space A to space B so it transforms from space B to space A.
 * Specialization for UE::Math::TQuat<T>.
 * 
 * @param Transform Input transform from space A to space B.
 * @return Inverted transform from space B to space A.
 */
template<typename T>
inline UE::Math::TQuat<T> Inverse(const UE::Math::TQuat<T>& Transform)
{
	return Transform.Inverse();
}

/**
 * Inverts a transform from space A to space B so it transforms from space B to space A.
 * Specialization for translation.
 * 
 * @param Transform Input transform from space A to space B.
 * @return Inverted transform from space B to space A.
 */
template<typename T>
inline UE::Math::TVector<T> Inverse(const UE::Math::TVector<T>& Transform)
{
	return -Transform;
}

//////////////////////////////////////////////////////////////////////////
// TransformPoint overloads. 
// 
// Since these are existing UE4 types, we cannot rely on the default
// template that calls member functions. Instead, we provide direct overloads.
//////////////////////////////////////////////////////////////////////////

/**
 * Specialization for UE::Math::TMatrix<T> as it's member function is called something slightly different.
 */
template<typename T>
inline UE::Math::TVector<T> TransformPoint(const UE::Math::TMatrix<T>& Transform, const UE::Math::TVector<T>& Point)
{
	return Transform.TransformPosition(Point);
}

/**
 * Specialization for UE::Math::TQuat<T> as it's member function is called something slightly different.
 */
template<typename T>
inline UE::Math::TVector<T> TransformPoint(const UE::Math::TQuat<T>& Transform, const UE::Math::TVector<T>& Point)
{
	return Transform.RotateVector(Point);
}

/**
 * Specialization for UE::Math::TQuat<T> as it's member function is called something slightly different.
 */
template<typename T>
inline UE::Math::TVector<T> TransformVector(const UE::Math::TQuat<T>& Transform, const UE::Math::TVector<T>& Vector)
{
	return Transform.RotateVector(Vector);
}

/**
 * Specialization for UE::Math::TRotator<T> as it's member function is called something slightly different.
 */
template<typename T>
inline UE::Math::TVector<T> TransformPoint(const UE::Math::TRotator<T>& Transform, const UE::Math::TVector<T>& Point)
{
	return Transform.RotateVector(Point);
}

/**
 * Specialization for UE::Math::TRotator<T> as it's member function is called something slightly different.
 */
template<typename T>
inline UE::Math::TVector<T> TransformVector(const UE::Math::TRotator<T>& Transform, const UE::Math::TVector<T>& Vector)
{
	return Transform.RotateVector(Vector);
}

/**
 * Specialization for UE::Math::TVector<T> Translation.
 */
template<typename T>
inline UE::Math::TVector<T> TransformPoint(const UE::Math::TVector<T>& Transform, const UE::Math::TVector<T>& Point)
{
	return Transform + Point;
}

/**
 * Specialization for UE::Math::TVector<T> Translation (does nothing).
 */
template<typename T>
inline const UE::Math::TVector<T>& TransformVector(const UE::Math::TVector<T>& Transform, const UE::Math::TVector<T>& Vector)
{
	return Vector;
}

/**
 * Specialization for Scale.
 */
template<typename T>
inline UE::Math::TVector<T> TransformPoint(const TScale<T>& Transform, const UE::Math::TVector<T>& Point)
{
	return Transform.GetVector() * Point;
}

/**
 * Specialization for Scale.
 */
template<typename T>
inline UE::Math::TVector<T> TransformVector(const TScale<T>& Transform, const UE::Math::TVector<T>& Vector)
{
	return Transform.GetVector() * Vector;
}

