// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Math/VectorRegister.h"
#include "Math/ScalarRegister.h"

#if !ENABLE_VECTORIZED_TRANSFORM

/**
* Transform composed of Scale, Rotation (as a quaternion), and Translation.
*
* Transforms can be used to convert from one space to another, for example by transforming
* positions and directions from local space to world space.
*
* Transformation of position vectors is applied in the order:  Scale -> Rotate -> Translate.
* Transformation of direction vectors is applied in the order: Scale -> Rotate.
*
* Order matters when composing transforms: C = A * B will yield a transform C that logically
* first applies A then B to any subsequent transformation. Note that this is the opposite order of quaternion (TQuat<T>) multiplication.
*
* Example: LocalToWorld = (DeltaRotation * LocalToWorld) will change rotation in local space by DeltaRotation.
* Example: LocalToWorld = (LocalToWorld * DeltaRotation) will change rotation in world space by DeltaRotation.
*/
struct Z_Construct_UScriptStruct_FTransform3f_Statics;
struct Z_Construct_UScriptStruct_FTransform3d_Statics;
struct Z_Construct_UScriptStruct_FTransform_Statics;

namespace UE
{
namespace Math
{

template<typename T>
struct alignas(alignof(TQuat<T>)) TTransform
{
	friend Z_Construct_UScriptStruct_FTransform3f_Statics;
	friend Z_Construct_UScriptStruct_FTransform3d_Statics;
	friend Z_Construct_UScriptStruct_FTransform_Statics;

	using FReal = T;
	using TransformVectorRegister = TVectorRegisterType<T>;

protected:
	/** Rotation of this transformation, as a quaternion. */
	TQuat<T>	Rotation;
	/** Translation of this transformation, as a vector. */
	TVector<T>	Translation;
	/** 3D scale (always applied in local space) as a vector. */
	TVector<T>	Scale3D;
public:
	/**
	* The identity transformation (Rotation = TQuat<T>::Identity, Translation = TVector<T>::ZeroVector, Scale3D = (1,1,1)).
	*/
	CORE_API static const TTransform<T> Identity;

#if ENABLE_NAN_DIAGNOSTIC
	inline void DiagnosticCheckNaN_Scale3D() const
	{
		if (Scale3D.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("TTransform<T> Scale3D contains NaN: %s"), *Scale3D.ToString());
		}
	}

	inline void DiagnosticCheckNaN_Translate() const
	{
		if (Translation.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("TTransform<T> Translation contains NaN: %s"), *Translation.ToString());
		}
	}

	inline void DiagnosticCheckNaN_Rotate() const
	{
		if (Rotation.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("TTransform<T> Rotation contains NaN: %s"), *Rotation.ToString());
		}
	}

	inline void DiagnosticCheckNaN_All() const
	{
		DiagnosticCheckNaN_Scale3D();
		DiagnosticCheckNaN_Rotate();
		DiagnosticCheckNaN_Translate();
	}

	inline void DiagnosticCheck_IsValid() const
	{
		DiagnosticCheckNaN_All();
		if (!IsValid())
		{
			logOrEnsureNanError(TEXT("TTransform<T> transform is not valid: %s"), *ToHumanReadableString());
		}

	}
#else
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN_Translate() const {}
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN_Rotate() const {}
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN_Scale3D() const {}
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN_All() const {}
	UE_FORCEINLINE_HINT void DiagnosticCheck_IsValid() const {}
#endif

	/** Default constructor. */
	[[nodiscard]] inline TTransform()
		: Rotation(0.f, 0.f, 0.f, 1.f)
		, Translation(0.f)
		, Scale3D(TVector<T>::OneVector)
	{
	}

	/**
	* Constructor with an initial translation
	*
	* @param InTranslation The value to use for the translation component
	*/
	[[nodiscard]] inline explicit TTransform(const TVector<T>& InTranslation)
		: Rotation(TQuat<T>::Identity),
		Translation(InTranslation),
		Scale3D(TVector<T>::OneVector)
	{
		DiagnosticCheckNaN_All();
	}

	/**
	* Constructor with leaving uninitialized memory
	*/
	[[nodiscard]] inline explicit TTransform(ENoInit)
	{
		// Note: This can be used to track down initialization issues with bone transform arrays; but it will
		// cause issues with transient fields such as RootMotionDelta that get initialized to 0 by default
#if ENABLE_NAN_DIAGNOSTIC
		// It is unsafe to call normal constructors here as they do their own NaN checks, so would always hit a false positive
		T qnan = FMath::Log2(-5.3f);
		check(FMath::IsNaN(qnan));
		Translation.X = Translation.Y = Translation.Z = qnan;
		Rotation.X = Rotation.Y = Rotation.Z = Rotation.W = qnan;
		Scale3D.X = Scale3D.Y = Scale3D.Z = qnan;
#endif
	}

	/**
	* Constructor with an initial rotation
	*
	* @param InRotation The value to use for rotation component
	*/
	[[nodiscard]] inline explicit TTransform(const TQuat<T>& InRotation)
		: Rotation(InRotation),
		Translation(TVector<T>::ZeroVector),
		Scale3D(TVector<T>::OneVector)
	{
		DiagnosticCheckNaN_All();
	}

	/**
	* Constructor with an initial rotation
	*
	* @param InRotation The value to use for rotation component  (after being converted to a quaternion)
	*/
	[[nodiscard]] inline explicit TTransform(const TRotator<T>& InRotation)
		: Rotation(InRotation),
		Translation(TVector<T>::ZeroVector),
		Scale3D(TVector<T>::OneVector)
	{
		DiagnosticCheckNaN_All();
	}

	/**
	* Constructor with all components initialized
	*
	* @param InRotation The value to use for rotation component
	* @param InTranslation The value to use for the translation component
	* @param InScale3D The value to use for the scale component
	*/
	[[nodiscard]] inline TTransform(const TQuat<T>& InRotation, const TVector<T>& InTranslation, const TVector<T>& InScale3D = TVector<T>::OneVector)
		: Rotation(InRotation),
		Translation(InTranslation),
		Scale3D(InScale3D)
	{
		DiagnosticCheckNaN_All();
	}

	/**
	* Constructor with all components initialized, taking a TRotator<T> as the rotation component
	*
	* @param InRotation The value to use for rotation component (after being converted to a quaternion)
	* @param InTranslation The value to use for the translation component
	* @param InScale3D The value to use for the scale component
	*/
	[[nodiscard]] inline TTransform(const TRotator<T>& InRotation, const TVector<T>& InTranslation, const TVector<T>& InScale3D = TVector<T>::OneVector)
		: Rotation(InRotation),
		Translation(InTranslation),
		Scale3D(InScale3D)
	{
		DiagnosticCheckNaN_All();
	}

	/**
	* Constructor for converting a Matrix (including scale) into a TTransform<T>.
	*/
	[[nodiscard]] inline explicit TTransform(const TMatrix<T>& InMatrix)
	{
		SetFromMatrix(InMatrix);
		DiagnosticCheckNaN_All();
	}

	/** Constructor that takes basis axes and translation */
	[[nodiscard]] inline TTransform(const TVector<T>& InX, const TVector<T>& InY, const TVector<T>& InZ, const TVector<T>& InTranslation)
	{
		SetFromMatrix(TMatrix<T>(InX, InY, InZ, InTranslation));
		DiagnosticCheckNaN_All();
	}

	/**
	* Does a debugf of the contents of this Transform.
	*/
	CORE_API void DebugPrint() const;

	/** Debug purpose only **/
	[[nodiscard]] bool DebugEqualMatrix(const TMatrix<T>& Matrix) const;

	/** Convert TTransform<T> contents to a string */
	[[nodiscard]] CORE_API FString ToHumanReadableString() const;

	[[nodiscard]] CORE_API FString ToString() const;

	/** Acceptable form: "%f,%f,%f|%f,%f,%f|%f,%f,%f" */
	CORE_API bool InitFromString(const FString& InSourceString);

	/**
	* Convert this Transform to a transformation matrix with scaling.
	*/
	[[nodiscard]] inline TMatrix<T> ToMatrixWithScale() const
	{
		TMatrix<T> OutMatrix;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_EDITORONLY_DATA
		// Make sure Rotation is normalized when we turn it into a matrix.
		check(IsRotationNormalized());
#endif
		OutMatrix.M[3][0] = Translation.X;
		OutMatrix.M[3][1] = Translation.Y;
		OutMatrix.M[3][2] = Translation.Z;

		const FReal x2 = Rotation.X + Rotation.X;
		const FReal y2 = Rotation.Y + Rotation.Y;
		const FReal z2 = Rotation.Z + Rotation.Z;
		{
			const FReal xx2 = Rotation.X * x2;
			const FReal yy2 = Rotation.Y * y2;
			const FReal zz2 = Rotation.Z * z2;

			OutMatrix.M[0][0] = (1.0f - (yy2 + zz2)) * Scale3D.X;
			OutMatrix.M[1][1] = (1.0f - (xx2 + zz2)) * Scale3D.Y;
			OutMatrix.M[2][2] = (1.0f - (xx2 + yy2)) * Scale3D.Z;
		}
		{
			const FReal yz2 = Rotation.Y * z2;
			const FReal wx2 = Rotation.W * x2;

			OutMatrix.M[2][1] = (yz2 - wx2) * Scale3D.Z;
			OutMatrix.M[1][2] = (yz2 + wx2) * Scale3D.Y;
		}
		{
			const FReal xy2 = Rotation.X * y2;
			const FReal wz2 = Rotation.W * z2;

			OutMatrix.M[1][0] = (xy2 - wz2) * Scale3D.Y;
			OutMatrix.M[0][1] = (xy2 + wz2) * Scale3D.X;
		}
		{
			const FReal xz2 = Rotation.X * z2;
			const FReal wy2 = Rotation.W * y2;

			OutMatrix.M[2][0] = (xz2 + wy2) * Scale3D.Z;
			OutMatrix.M[0][2] = (xz2 - wy2) * Scale3D.X;
		}

		OutMatrix.M[0][3] = 0.0f;
		OutMatrix.M[1][3] = 0.0f;
		OutMatrix.M[2][3] = 0.0f;
		OutMatrix.M[3][3] = 1.0f;

		return OutMatrix;
	}

	/**
	* Convert this Transform to matrix with scaling and compute the inverse of that.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TMatrix<T> ToInverseMatrixWithScale() const
	{
		// todo: optimize
		return ToMatrixWithScale().Inverse();
	}

	/**
	* Convert this Transform to inverse.
	*/
	[[nodiscard]] inline TTransform<T> Inverse() const
	{
		TQuat<T>   InvRotation = Rotation.Inverse();
		// this used to cause NaN if Scale contained 0 
		TVector<T> InvScale3D = GetSafeScaleReciprocal(Scale3D);
		TVector<T> InvTranslation = InvRotation * (InvScale3D * -Translation);

		return TTransform(InvRotation, InvTranslation, InvScale3D);
	}

	/**
	* Convert this Transform to a transformation matrix, ignoring its scaling
	*/
	[[nodiscard]] inline TMatrix<T> ToMatrixNoScale() const
	{
		TMatrix<T> OutMatrix;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_EDITORONLY_DATA
		// Make sure Rotation is normalized when we turn it into a matrix.
		check(IsRotationNormalized());
#endif
		OutMatrix.M[3][0] = Translation.X;
		OutMatrix.M[3][1] = Translation.Y;
		OutMatrix.M[3][2] = Translation.Z;

		const FReal x2 = Rotation.X + Rotation.X;
		const FReal y2 = Rotation.Y + Rotation.Y;
		const FReal z2 = Rotation.Z + Rotation.Z;
		{
			const FReal xx2 = Rotation.X * x2;
			const FReal yy2 = Rotation.Y * y2;
			const FReal zz2 = Rotation.Z * z2;

			OutMatrix.M[0][0] = (1.0f - (yy2 + zz2));
			OutMatrix.M[1][1] = (1.0f - (xx2 + zz2));
			OutMatrix.M[2][2] = (1.0f - (xx2 + yy2));
		}
		{
			const FReal yz2 = Rotation.Y * z2;
			const FReal wx2 = Rotation.W * x2;

			OutMatrix.M[2][1] = (yz2 - wx2);
			OutMatrix.M[1][2] = (yz2 + wx2);
		}
		{
			const FReal xy2 = Rotation.X * y2;
			const FReal wz2 = Rotation.W * z2;

			OutMatrix.M[1][0] = (xy2 - wz2);
			OutMatrix.M[0][1] = (xy2 + wz2);
		}
		{
			const FReal xz2 = Rotation.X * z2;
			const FReal wy2 = Rotation.W * y2;

			OutMatrix.M[2][0] = (xz2 + wy2);
			OutMatrix.M[0][2] = (xz2 - wy2);
		}

		OutMatrix.M[0][3] = 0.0f;
		OutMatrix.M[1][3] = 0.0f;
		OutMatrix.M[2][3] = 0.0f;
		OutMatrix.M[3][3] = 1.0f;

		return OutMatrix;
	}

	/** Set this transform to the weighted blend of the supplied two transforms. */
	inline void Blend(const TTransform<T>& Atom1, const TTransform<T>& Atom2, float Alpha)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_EDITORONLY_DATA
		// Check that all bone atoms coming from animation are normalized
		check(Atom1.IsRotationNormalized());
		check(Atom2.IsRotationNormalized());
#endif
		if (Alpha <= ZERO_ANIMWEIGHT_THRESH)
		{
			// if blend is all the way for child1, then just copy its bone atoms
			(*this) = Atom1;
		}
		else if (Alpha >= 1.f - ZERO_ANIMWEIGHT_THRESH)
		{
			// if blend is all the way for child2, then just copy its bone atoms
			(*this) = Atom2;
		}
		else
		{
			// Simple linear interpolation for translation and scale.
			Translation = FMath::Lerp(Atom1.Translation, Atom2.Translation, Alpha);
			Scale3D = FMath::Lerp(Atom1.Scale3D, Atom2.Scale3D, Alpha);
			Rotation = TQuat<T>::FastLerp(Atom1.Rotation, Atom2.Rotation, Alpha);

			// ..and renormalize
			Rotation.Normalize();
		}
	}

	/** Set this Transform to the weighted blend of it and the supplied Transform. */
	inline void BlendWith(const TTransform<T>& OtherAtom, float Alpha)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_EDITORONLY_DATA
		// Check that all bone atoms coming from animation are normalized
		check(IsRotationNormalized());
		check(OtherAtom.IsRotationNormalized());
#endif
		if (Alpha > ZERO_ANIMWEIGHT_THRESH)
		{
			if (Alpha >= 1.f - ZERO_ANIMWEIGHT_THRESH)
			{
				// if blend is all the way for child2, then just copy its bone atoms
				(*this) = OtherAtom;
			}
			else
			{
				// Simple linear interpolation for translation and scale.
				Translation = FMath::Lerp(Translation, OtherAtom.Translation, Alpha);
				Scale3D = FMath::Lerp(Scale3D, OtherAtom.Scale3D, Alpha);
				Rotation = TQuat<T>::FastLerp(Rotation, OtherAtom.Rotation, Alpha);

				// ..and renormalize
				Rotation.Normalize();
			}
		}
	}


	/**
	* Quaternion addition is wrong here. This is just a special case for linear interpolation.
	* Use only within blends!!
	* Rotation part is NOT normalized!!
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TTransform<T> operator+(const TTransform<T>& Atom) const
	{
		return TTransform(Rotation + Atom.Rotation, Translation + Atom.Translation, Scale3D + Atom.Scale3D);
	}

	inline TTransform<T>& operator+=(const TTransform<T>& Atom)
	{
		Translation += Atom.Translation;

		Rotation.X += Atom.Rotation.X;
		Rotation.Y += Atom.Rotation.Y;
		Rotation.Z += Atom.Rotation.Z;
		Rotation.W += Atom.Rotation.W;

		Scale3D += Atom.Scale3D;

		return *this;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT TTransform<T> operator*(T Mult) const
	{
		return TTransform(Rotation * Mult, Translation * Mult, Scale3D * Mult);
	}

	inline TTransform<T>& operator*=(T Mult)
	{
		Translation *= Mult;
		Rotation.X *= Mult;
		Rotation.Y *= Mult;
		Rotation.Z *= Mult;
		Rotation.W *= Mult;
		Scale3D *= Mult;

		return *this;
	}

	/**
	* Return a transform that is the result of this multiplied by another transform.
	* Order matters when composing transforms : C = A * B will yield a transform C that logically first applies A then B to any subsequent transformation.
	*
	* @param  Other other transform by which to multiply.
	* @return new transform: this * Other
	*/
	[[nodiscard]] inline TTransform<T> operator*(const TTransform<T>& Other) const;


	/**
	* Sets this transform to the result of this multiplied by another transform.
	* Order matters when composing transforms : C = A * B will yield a transform C that logically first applies A then B to any subsequent transformation.
	*
	* @param  Other other transform by which to multiply.
	*/
	UE_FORCEINLINE_HINT void operator*=(const TTransform<T>& Other);

	/**
	* Return a transform that is the result of this multiplied by another transform (made only from a rotation).
	* Order matters when composing transforms : C = A * B will yield a transform C that logically first applies A then B to any subsequent transformation.
	*
	* @param  Other other quaternion rotation by which to multiply.
	* @return new transform: this * TTransform(Other)
	*/
	[[nodiscard]] inline TTransform<T> operator*(const TQuat<T>& Other) const;

	/**
	* Sets this transform to the result of this multiplied by another transform (made only from a rotation).
	* Order matters when composing transforms : C = A * B will yield a transform C that logically first applies A then B to any subsequent transformation.
	*
	* @param  Other other quaternion rotation by which to multiply.
	*/
	inline void operator*=(const TQuat<T>& Other);

	[[nodiscard]] inline static bool AnyHasNegativeScale(const TVector<T>& InScale3D, const TVector<T>& InOtherScale3D);
	inline void ScaleTranslation(const TVector<T>& InScale3D);
	inline void ScaleTranslation(const FReal& Scale);
	inline void RemoveScaling(FReal Tolerance = UE_SMALL_NUMBER);
	[[nodiscard]] inline T GetMaximumAxisScale() const;
	[[nodiscard]] inline T GetMinimumAxisScale() const;

	// Inverse does not work well with VQS format(in particular non-uniform), so removing it, but made two below functions to be used instead. 

	/*******************************************************************************************
	* The below 2 functions are the ones to get delta transform and return TTransform<T> format that can be concatenated
	* Inverse itself can't concatenate with VQS format(since VQS always transform from S->Q->T, where inverse happens from T(-1)->Q(-1)->S(-1))
	* So these 2 provides ways to fix this
	* GetRelativeTransform returns this*Other(-1) and parameter is Other(not Other(-1))
	* GetRelativeTransformReverse returns this(-1)*Other, and parameter is Other.
	*******************************************************************************************/
	[[nodiscard]] CORE_API TTransform<T> GetRelativeTransform(const TTransform<T>& Other) const;
	[[nodiscard]] CORE_API TTransform<T> GetRelativeTransformReverse(const TTransform<T>& Other) const;
	/**
	* Set current transform and the relative to ParentTransform.
	* Equates to This = This->GetRelativeTransform(Parent), but saves the intermediate TTransform<T> storage and copy.
	*/
	CORE_API void SetToRelativeTransform(const TTransform<T>& ParentTransform);

	[[nodiscard]] inline TVector4<T> TransformFVector4(const TVector4<T>& V) const;
	[[nodiscard]] inline TVector4<T> TransformFVector4NoScale(const TVector4<T>& V) const;
	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> TransformPosition(const TVector<T>& V) const;
	[[nodiscard]] inline TVector<T> TransformPositionNoScale(const TVector<T>& V) const;

	/** Inverts the transform and then transforms V - correctly handles scaling in this transform. */
	[[nodiscard]] inline TVector<T> InverseTransformPosition(const TVector<T> &V) const;
	[[nodiscard]] inline TVector<T> InverseTransformPositionNoScale(const TVector<T> &V) const;
	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> TransformVector(const TVector<T>& V) const;
	[[nodiscard]] inline TVector<T> TransformVectorNoScale(const TVector<T>& V) const;

	/**
	*	Transform a direction vector by the inverse of this transform - will not take into account translation part.
	*	If you want to transform a surface normal (or plane) and correctly account for non-uniform scaling you should use TransformByUsingAdjointT with adjoint of matrix inverse.
	*/
	[[nodiscard]] inline TVector<T> InverseTransformVector(const TVector<T> &V) const;
	[[nodiscard]] inline TVector<T> InverseTransformVectorNoScale(const TVector<T> &V) const;

	/**
	 * Transform a rotation.
	 * For example if this is a LocalToWorld transform, TransformRotation(Q) would transform Q from local to world space.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TQuat<T> TransformRotation(const TQuat<T>& Q) const;
	
	/**
	* Inverse transform a rotation.
	* For example if this is a LocalToWorld transform, InverseTransformRotation(Q) would transform Q from world to local space.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TQuat<T> InverseTransformRotation(const TQuat<T>& Q) const;

	[[nodiscard]] inline TTransform<T> GetScaled(T Scale) const;
	[[nodiscard]] inline TTransform<T> GetScaled(TVector<T> Scale) const;
	[[nodiscard]] inline TVector<T> GetScaledAxis(EAxis::Type InAxis) const;
	[[nodiscard]] inline TVector<T> GetUnitAxis(EAxis::Type InAxis) const;
	[[nodiscard]] inline void Mirror(EAxis::Type MirrorAxis, EAxis::Type FlipAxis);
	[[nodiscard]] inline static TVector<T> GetSafeScaleReciprocal(const TVector<T>& InScale, FReal Tolerance = UE_SMALL_NUMBER);

	// temp function for easy conversion
	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> GetLocation() const
	{
		return GetTranslation();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT TRotator<T> Rotator() const
	{
		return Rotation.Rotator();
	}

	/** Calculate the determinant of this transformation */
	[[nodiscard]] UE_FORCEINLINE_HINT FReal GetDeterminant() const
	{
		return (float)(Scale3D.X * Scale3D.Y * Scale3D.Z);
	}

	/** Set the translation of this transformation */
	inline void SetLocation(const TVector<T>& Origin)
	{
		Translation = Origin;
		DiagnosticCheckNaN_Translate();
	}

	/**
	* Checks the components for non-finite values (NaN or Inf).
	* @return Returns true if any component (rotation, translation, or scale) is not finite.
	*/
	[[nodiscard]] bool ContainsNaN() const
	{
		return (Translation.ContainsNaN() || Rotation.ContainsNaN() || Scale3D.ContainsNaN());
	}

	[[nodiscard]] inline bool IsValid() const
	{
		if (ContainsNaN())
		{
			return false;
		}

		if (!Rotation.IsNormalized())
		{
			return false;
		}

		return true;
	}

	// Serializer.
	inline friend FArchive& operator<<(FArchive& Ar, TTransform<T>& M)
	{
		Ar << M.Rotation;
		Ar << M.Translation;
		Ar << M.Scale3D;
		return Ar;
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		// FTransform3f/d are immutable, but FTransform is not and needs a per-property conversion
		if(StructTag == NAME_Transform)
		{
			return false;
		}

		if constexpr (std::is_same_v<T, float>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Transform, Transform3f, Transform3d);
		}
		else
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Transform, Transform3d, Transform3f);
		}
	}

	// Binary comparison operators.
	/*
	bool operator==(const TTransform<T>& Other) const
	{
	return Rotation==Other.Rotation && Translation==Other.Translation && Scale3D==Other.Scale3D;
	}
	bool operator!=(const TTransform<T>& Other) const
	{
	return Rotation!=Other.Rotation || Translation!=Other.Translation || Scale3D!=Other.Scale3D;
	}
	*/

private:

	UE_FORCEINLINE_HINT bool Private_RotationEquals(const TQuat<T>& InRotation, const FReal Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return Rotation.Equals(InRotation, Tolerance);
	}

	UE_FORCEINLINE_HINT bool Private_TranslationEquals(const TVector<T>& InTranslation, const FReal Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return Translation.Equals(InTranslation, Tolerance);
	}

	UE_FORCEINLINE_HINT bool Private_Scale3DEquals(const TVector<T>& InScale3D, const FReal Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return Scale3D.Equals(InScale3D, Tolerance);
	}

public:

	// Test if A's rotation equals B's rotation, within a tolerance. Preferred over "A.GetRotation().Equals(B.GetRotation())" because it is faster on some platforms.
	[[nodiscard]] UE_FORCEINLINE_HINT static bool AreRotationsEqual(const TTransform<T>& A, const TTransform<T>& B, FReal Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return A.Private_RotationEquals(B.Rotation, Tolerance);
	}

	// Test if A's translation equals B's translation, within a tolerance. Preferred over "A.GetTranslation().Equals(B.GetTranslation())" because it is faster on some platforms.
	[[nodiscard]] UE_FORCEINLINE_HINT static bool AreTranslationsEqual(const TTransform<T>& A, const TTransform<T>& B, FReal Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return A.Private_TranslationEquals(B.Translation, Tolerance);
	}

	// Test if A's scale equals B's scale, within a tolerance. Preferred over "A.GetScale3D().Equals(B.GetScale3D())" because it is faster on some platforms.
	[[nodiscard]] UE_FORCEINLINE_HINT static bool AreScale3DsEqual(const TTransform<T>& A, const TTransform<T>& B, FReal Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return A.Private_Scale3DEquals(B.Scale3D, Tolerance);
	}



	// Test if this Transform's rotation equals another's rotation, within a tolerance. Preferred over "GetRotation().Equals(Other.GetRotation())" because it is faster on some platforms.
	[[nodiscard]] UE_FORCEINLINE_HINT bool RotationEquals(const TTransform<T>& Other, FReal Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return AreRotationsEqual(*this, Other, Tolerance);
	}

	// Test if this Transform's translation equals another's translation, within a tolerance. Preferred over "GetTranslation().Equals(Other.GetTranslation())" because it is faster on some platforms.
	[[nodiscard]] UE_FORCEINLINE_HINT bool TranslationEquals(const TTransform<T>& Other, FReal Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return AreTranslationsEqual(*this, Other, Tolerance);
	}

	// Test if this Transform's scale equals another's scale, within a tolerance. Preferred over "GetScale3D().Equals(Other.GetScale3D())" because it is faster on some platforms.
	[[nodiscard]] UE_FORCEINLINE_HINT bool Scale3DEquals(const TTransform<T>& Other, FReal Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return AreScale3DsEqual(*this, Other, Tolerance);
	}

	// Test if all components of the transforms are equal, within a tolerance.
	[[nodiscard]] UE_FORCEINLINE_HINT bool Equals(const TTransform<T>& Other, FReal Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return Private_TranslationEquals(Other.Translation, Tolerance) && Private_RotationEquals(Other.Rotation, Tolerance) && Private_Scale3DEquals(Other.Scale3D, Tolerance);
	}

	// Test if all components of the transform property are equal.
	[[nodiscard]] UE_FORCEINLINE_HINT bool Identical(const TTransform<T>* Other, uint32 PortFlags) const
	{
		return Equals(*Other, 0.f);
	}

	// Test if rotation and translation components of the transforms are equal, within a tolerance.
	[[nodiscard]] UE_FORCEINLINE_HINT bool EqualsNoScale(const TTransform<T>& Other, FReal Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return Private_TranslationEquals(Other.Translation, Tolerance) && Private_RotationEquals(Other.Rotation, Tolerance);
	}

	/**
	* Create a new transform: OutTransform = A * B.
	*
	* Order matters when composing transforms : A * B will yield a transform that logically first applies A then B to any subsequent transformation.
	*
	* @param  OutTransform pointer to transform that will store the result of A * B.
	* @param  A Transform A.
	* @param  B Transform B.
	*/
	UE_FORCEINLINE_HINT static void Multiply(TTransform<T>* OutTransform, const TTransform<T>* A, const TTransform<T>* B);

	/**
	* Sets the components
	* @param InRotation The new value for the Rotation component
	* @param InTranslation The new value for the Translation component
	* @param InScale3D The new value for the Scale3D component
	*/
	inline void SetComponents(const TQuat<T>& InRotation, const TVector<T>& InTranslation, const TVector<T>& InScale3D)
	{
		Rotation = InRotation;
		Translation = InTranslation;
		Scale3D = InScale3D;

		DiagnosticCheckNaN_All();
	}

	/**
	* Sets the components to the identity transform:
	*   Rotation = (0,0,0,1)
	*   Translation = (0,0,0)
	*   Scale3D = (1,1,1)
	*/
	inline void SetIdentity()
	{
		Rotation = TQuat<T>::Identity;
		Translation = TVector<T>::ZeroVector;
		Scale3D = TVector<T>(1, 1, 1);
	}

	/**
	* Sets the components to the 'additive' identity transform:
	*   Rotation = (0,0,0,1)
	*   Translation = (0,0,0)
	*   Scale3D = (0,0,0)
	*/
	inline void SetIdentityZeroScale()
	{
		Rotation = TQuat<T>::Identity;
		Translation = TVector<T>::ZeroVector;
		Scale3D = TVector<T>::ZeroVector;
	}
	
	/**
	* Scales the Scale3D component by a new factor
	* @param Scale3DMultiplier The value to multiply Scale3D with
	*/
	inline void MultiplyScale3D(const TVector<T>& Scale3DMultiplier)
	{
		Scale3D *= Scale3DMultiplier;
		DiagnosticCheckNaN_Scale3D();
	}

	/**
	* Sets the translation component
	* @param NewTranslation The new value for the translation component
	*/
	inline void SetTranslation(const TVector<T>& NewTranslation)
	{
		Translation = NewTranslation;
		DiagnosticCheckNaN_Translate();
	}

	/** Copy translation from another TTransform<T>. */
	UE_FORCEINLINE_HINT void CopyTranslation(const TTransform<T>& Other)
	{
		Translation = Other.Translation;
	}

	/**
	* Concatenates another rotation to this transformation
	* @param DeltaRotation The rotation to concatenate in the following fashion: Rotation = Rotation * DeltaRotation
	*/
	inline void ConcatenateRotation(const TQuat<T>& DeltaRotation)
	{
		Rotation = Rotation * DeltaRotation;
		DiagnosticCheckNaN_Rotate();
	}

	/**
	* Adjusts the translation component of this transformation
	* @param DeltaTranslation The translation to add in the following fashion: Translation += DeltaTranslation
	*/
	inline void AddToTranslation(const TVector<T>& DeltaTranslation)
	{
		Translation += DeltaTranslation;
		DiagnosticCheckNaN_Translate();
	}

	/**
	* Add the translations from two TTransform<T>s and return the result.
	* @return A.Translation + B.Translation
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT static TVector<T> AddTranslations(const TTransform<T>& A, const TTransform<T>& B)
	{
		return A.Translation + B.Translation;
	}

	/**
	* Subtract translations from two TTransform<T>s and return the difference.
	* @return A.Translation - B.Translation.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT static TVector<T> SubtractTranslations(const TTransform<T>& A, const TTransform<T>& B)
	{
		return A.Translation - B.Translation;
	}

	/**
	* Sets the rotation component
	* @param NewRotation The new value for the rotation component
	*/
	inline void SetRotation(const TQuat<T>& NewRotation)
	{
		Rotation = NewRotation;
		DiagnosticCheckNaN_Rotate();
	}

	/** Copy rotation from another TTransform<T>. */
	UE_FORCEINLINE_HINT void CopyRotation(const TTransform<T>& Other)
	{
		Rotation = Other.Rotation;
	}

	/**
	* Sets the Scale3D component
	* @param NewScale3D The new value for the Scale3D component
	*/
	inline void SetScale3D(const TVector<T>& NewScale3D)
	{
		Scale3D = NewScale3D;
		DiagnosticCheckNaN_Scale3D();
	}

	/** Copy scale from another TTransform<T>. */
	UE_FORCEINLINE_HINT void CopyScale3D(const TTransform<T>& Other)
	{
		Scale3D = Other.Scale3D;
	}

	/**
	* Sets both the translation and Scale3D components at the same time
	* @param NewTranslation The new value for the translation component
	* @param NewScale3D The new value for the Scale3D component
	*/
	inline void SetTranslationAndScale3D(const TVector<T>& NewTranslation, const TVector<T>& NewScale3D)
	{
		Translation = NewTranslation;
		Scale3D = NewScale3D;

		DiagnosticCheckNaN_Translate();
		DiagnosticCheckNaN_Scale3D();
	}

	// For low-level VectorRegister programming
	[[nodiscard]] TransformVectorRegister GetTranslationRegister() const
	{
		return VectorLoadFloat3_W0(&Translation);
	}
	[[nodiscard]] TransformVectorRegister GetRotationRegister() const
	{
		return VectorLoad(&Rotation);
	}
	void SetTranslationRegister(TransformVectorRegister InTranslation)
	{
		VectorStoreFloat3(InTranslation, &Translation);
	}
	void SetRotationRegister(TransformVectorRegister InRotation)
	{
		VectorStore(InRotation, &Rotation);
	}

	/** @note: Added template type function for Accumulate
	* The template type isn't much useful yet, but it is with the plan to move forward
	* to unify blending features with just type of additive or full pose
	* Eventually it would be nice to just call blend and it all works depending on full pose
	* or additive, but right now that is a lot more refactoring
	* For now this types only defines the different functionality of accumulate
	*/

	/**
	* Accumulates another transform with this one
	*
	* Rotation is accumulated multiplicatively (Rotation = SourceAtom.Rotation * Rotation)
	* Translation is accumulated additively (Translation += SourceAtom.Translation)
	* Scale3D is accumulated multiplicatively (Scale3D *= SourceAtom.Scale3D)
	*
	* @param SourceAtom The other transform to accumulate into this one
	*/
	inline void Accumulate(const TTransform<T>& SourceAtom)
	{
		Rotation = SourceAtom.Rotation * Rotation;
		Translation += SourceAtom.Translation;
		Scale3D *= SourceAtom.Scale3D;

		DiagnosticCheckNaN_All();

		checkSlow(IsRotationNormalized());
	}

	/** Accumulates another transform with this one, with a blending weight
	*
	* Let SourceAtom = Atom * BlendWeight
	* Rotation is accumulated multiplicatively (Rotation = SourceAtom.Rotation * Rotation).
	* Translation is accumulated additively (Translation += SourceAtom.Translation)
	* Scale3D is accumulated multiplicatively (Scale3D *= SourceAtom.Scale3D)
	*
	* Note: Rotation will not be normalized! Will have to be done manually.
	*
	* @param Atom The other transform to accumulate into this one
	* @param BlendWeight The weight to multiply Atom by before it is accumulated.
	*/
	inline void Accumulate(const TTransform<T>& Atom, FReal BlendWeight/* default param doesn't work since vectorized version takes ref param */)
	{
		TTransform<T> SourceAtom(Atom * BlendWeight);

		Rotation = SourceAtom.Rotation * Rotation;
		Translation += SourceAtom.Translation;
		Scale3D *= SourceAtom.Scale3D;

		DiagnosticCheckNaN_All();
	}

	/**
	* Accumulates another transform with this one, with an optional blending weight
	*
	* Rotation is accumulated additively, in the shortest direction (Rotation = Rotation +/- DeltaAtom.Rotation * Weight)
	* Translation is accumulated additively (Translation += DeltaAtom.Translation * Weight)
	* Scale3D is accumulated additively (Scale3D += DeltaAtom.Scale3D * Weight)
	*
	* @param DeltaAtom The other transform to accumulate into this one
	* @param Weight The weight to multiply DeltaAtom by before it is accumulated.
	*/
	inline void AccumulateWithShortestRotation(const TTransform<T>& DeltaAtom, FReal BlendWeight/* default param doesn't work since vectorized version takes ref param */)
	{
		TTransform<T> Atom(DeltaAtom * BlendWeight);

		// To ensure the 'shortest route', we make sure the dot product between the accumulator and the incoming child atom is positive.
		if ((Atom.Rotation | Rotation) < 0.f)
		{
			Rotation.X -= Atom.Rotation.X;
			Rotation.Y -= Atom.Rotation.Y;
			Rotation.Z -= Atom.Rotation.Z;
			Rotation.W -= Atom.Rotation.W;
		}
		else
		{
			Rotation.X += Atom.Rotation.X;
			Rotation.Y += Atom.Rotation.Y;
			Rotation.Z += Atom.Rotation.Z;
			Rotation.W += Atom.Rotation.W;
		}

		Translation += Atom.Translation;
		Scale3D += Atom.Scale3D;

		DiagnosticCheckNaN_All();
	}

	/** Accumulates another transform with this one, with a blending weight
	*
	* Let SourceAtom = Atom * BlendWeight
	* Rotation is accumulated multiplicatively (Rotation = SourceAtom.Rotation * Rotation).
	* Translation is accumulated additively (Translation += SourceAtom.Translation)
	* Scale3D is accumulated assuming incoming scale is additive scale (Scale3D *= (1 + SourceAtom.Scale3D))
	*
	* When we create additive, we create additive scale based on [TargetScale/SourceScale -1]
	* because that way when you apply weight of 0.3, you don't shrink. We only saves the % of grow/shrink
	* when we apply that back to it, we add back the 1, so that it goes back to it.
	* This solves issue where you blend two additives with 0.3, you don't come back to 0.6 scale, but 1 scale at the end
	* because [1 + [1-1]*0.3 + [1-1]*0.3] becomes 1, so you don't shrink by applying additive scale
	*
	* Note: Rotation will not be normalized! Will have to be done manually.
	*
	* @param Atom The other transform to accumulate into this one
	* @param BlendWeight The weight to multiply Atom by before it is accumulated.
	*/
	inline void AccumulateWithAdditiveScale(const TTransform<T>& Atom, T BlendWeight/* default param doesn't work since vectorized version takes ref param */)
	{
		const TVector<T> DefaultScale(TVector<T>::OneVector);

		TTransform<T> SourceAtom(Atom * BlendWeight);

		Rotation = SourceAtom.Rotation * Rotation;
		Translation += SourceAtom.Translation;
		Scale3D *= (DefaultScale + SourceAtom.Scale3D);

		DiagnosticCheckNaN_All();
	}
	/**
	* Set the translation and Scale3D components of this transform to a linearly interpolated combination of two other transforms
	*
	* Translation = FMath::Lerp(SourceAtom1.Translation, SourceAtom2.Translation, Alpha)
	* Scale3D = FMath::Lerp(SourceAtom1.Scale3D, SourceAtom2.Scale3D, Alpha)
	*
	* @param SourceAtom1 The starting point source atom (used 100% if Alpha is 0)
	* @param SourceAtom2 The ending point source atom (used 100% if Alpha is 1)
	* @param Alpha The blending weight between SourceAtom1 and SourceAtom2
	*/
	inline void LerpTranslationScale3D(const TTransform<T>& SourceAtom1, const TTransform<T>& SourceAtom2, ScalarRegister Alpha)
	{
		Translation = FMath::Lerp(SourceAtom1.Translation, SourceAtom2.Translation, Alpha);
		Scale3D = FMath::Lerp(SourceAtom1.Scale3D, SourceAtom2.Scale3D, Alpha);

		DiagnosticCheckNaN_Translate();
		DiagnosticCheckNaN_Scale3D();
	}

	/**
	* Normalize the rotation component of this transformation
	*/
	inline void NormalizeRotation()
	{
		Rotation.Normalize();
		DiagnosticCheckNaN_Rotate();
	}

	/**
	* Checks whether the rotation component is normalized or not
	*
	* @return true if the rotation component is normalized, and false otherwise.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsRotationNormalized() const
	{
		return Rotation.IsNormalized();
	}

	/**
	* Blends the Identity transform with a weighted source transform and accumulates that into a destination transform
	*
	* DeltaAtom = Blend(Identity, SourceAtom, BlendWeight)
	* FinalAtom.Rotation = DeltaAtom.Rotation * FinalAtom.Rotation
	* FinalAtom.Translation += DeltaAtom.Translation
	* FinalAtom.Scale3D *= DeltaAtom.Scale3D
	*
	* @param FinalAtom [in/out] The atom to accumulate the blended source atom into
	* @param SourceAtom The target transformation (used when BlendWeight = 1); this is modified during the process
	* @param BlendWeight The blend weight between Identity and SourceAtom
	*/
	inline static void BlendFromIdentityAndAccumulate(TTransform<T>& FinalAtom, const TTransform<T>& SourceAtom, float BlendWeight)
	{
		const TTransform<T> AdditiveIdentity(TQuat<T>::Identity, TVector<T>::ZeroVector, TVector<T>::ZeroVector);
		const TVector<T> DefaultScale(TVector<T>::OneVector);
		TTransform<T> DeltaAtom = SourceAtom;

		// Scale delta by weight
		if (BlendWeight < (1.f - ZERO_ANIMWEIGHT_THRESH))
		{
			DeltaAtom.Blend(AdditiveIdentity, DeltaAtom, BlendWeight);
		}

		FinalAtom.Rotation = DeltaAtom.Rotation * FinalAtom.Rotation;
		FinalAtom.Translation += DeltaAtom.Translation;
		FinalAtom.Scale3D *= (DefaultScale + DeltaAtom.Scale3D);

		checkSlow(FinalAtom.IsRotationNormalized());
	}

	/**
	* Returns the rotation component
	*
	* @return The rotation component
	*/
	[[nodiscard]] inline TQuat<T> GetRotation() const
	{
		DiagnosticCheckNaN_Rotate();
		return Rotation;
	}

	/**
	* Returns the translation component
	*
	* @return The translation component
	*/
	[[nodiscard]] inline TVector<T> GetTranslation() const
	{
		DiagnosticCheckNaN_Translate();
		return Translation;
	}

	/**
	* Returns the Scale3D component
	*
	* @return The Scale3D component
	*/
	[[nodiscard]] inline TVector<T> GetScale3D() const
	{
		DiagnosticCheckNaN_Scale3D();
		return Scale3D;
	}

	/**
	* Sets the Rotation and Scale3D of this transformation from another transform
	*
	* @param SrcBA The transform to copy rotation and Scale3D from
	*/
	inline void CopyRotationPart(const TTransform<T>& SrcBA)
	{
		Rotation = SrcBA.Rotation;
		Scale3D = SrcBA.Scale3D;

		DiagnosticCheckNaN_Rotate();
		DiagnosticCheckNaN_Scale3D();
	}

	/**
	* Sets the Translation and Scale3D of this transformation from another transform
	*
	* @param SrcBA The transform to copy translation and Scale3D from
	*/
	inline void CopyTranslationAndScale3D(const TTransform<T>& SrcBA)
	{
		Translation = SrcBA.Translation;
		Scale3D = SrcBA.Scale3D;

		DiagnosticCheckNaN_Translate();
		DiagnosticCheckNaN_Scale3D();
	}

	void SetFromMatrix(const TMatrix<T>& InMatrix)
	{
		TMatrix<T> M = InMatrix;

		// Get the 3D scale from the matrix
		Scale3D = M.ExtractScaling(0);

		// If there is negative scaling going on, we handle that here
		if (InMatrix.Determinant() < 0.f)
		{
			// Assume it is along X and modify transform accordingly. 
			// It doesn't actually matter which axis we choose, the 'appearance' will be the same
			Scale3D.X *= -1.f;
			M.SetAxis(0, -M.GetScaledAxis(EAxis::X));
		}

		Rotation = TQuat<T>(M);
		Translation = InMatrix.GetOrigin();

		// Normalize rotation
		Rotation.Normalize();
	}

private:
	/**
	* Create a new transform: OutTransform = A * B using the matrix while keeping the scale that's given by A and B
	* Please note that this operation is a lot more expensive than normal Multiply
	*
	* Order matters when composing transforms : A * B will yield a transform that logically first applies A then B to any subsequent transformation.
	*
	* @param  OutTransform pointer to transform that will store the result of A * B.
	* @param  A Transform A.
	* @param  B Transform B.
	*/
	UE_FORCEINLINE_HINT static void MultiplyUsingMatrixWithScale(TTransform<T>* OutTransform, const TTransform<T>* A, const TTransform<T>* B);
	/**
	* Create a new transform from multiplications of given to matrices (AMatrix*BMatrix) using desired scale
	* This is used by MultiplyUsingMatrixWithScale and GetRelativeTransformUsingMatrixWithScale
	* This is only used to handle negative scale
	*
	* @param	AMatrix first Matrix of operation
	* @param	BMatrix second Matrix of operation
	* @param	DesiredScale - there is no check on if the magnitude is correct here. It assumes that is correct.
	* @param	OutTransform the constructed transform
	*/
	UE_FORCEINLINE_HINT static void ConstructTransformFromMatrixWithDesiredScale(const TMatrix<T>& AMatrix, const TMatrix<T>& BMatrix, const TVector<T>& DesiredScale, TTransform<T>& OutTransform);
	/**
	* Create a new transform: OutTransform = Base * Relative(-1) using the matrix while keeping the scale that's given by Base and Relative
	* Please note that this operation is a lot more expensive than normal GetRelativeTrnasform
	*
	* @param  OutTransform pointer to transform that will store the result of Base * Relative(-1).
	* @param  BAse Transform Base.
	* @param  Relative Transform Relative.
	*/
	static void GetRelativeTransformUsingMatrixWithScale(TTransform<T>* OutTransform, const TTransform<T>* Base, const TTransform<T>* Relative);

public:
	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TTransform(const TTransform<FArg>& From)
		: TTransform<T>((TQuat<T>)From.GetRotation(), (TVector<T>)From.GetTranslation(), (TVector<T>)From.GetScale3D())
	{
	}
};

#if !defined(_MSC_VER) || defined(__clang__)  // MSVC can't forward declare explicit specializations
template<> CORE_API const FTransform3f FTransform3f::Identity;
template<> CORE_API const FTransform3d FTransform3d::Identity;
#endif
template<typename T>
inline bool TTransform<T>::AnyHasNegativeScale(const TVector<T>& InScale3D, const  TVector<T>& InOtherScale3D) 
{
	return  (InScale3D.X < 0.f || InScale3D.Y < 0.f || InScale3D.Z < 0.f 
			|| InOtherScale3D.X < 0.f || InOtherScale3D.Y < 0.f || InOtherScale3D.Z < 0.f );
}

/** Scale the translation part of the Transform by the supplied vector. */
template<typename T>
inline void TTransform<T>::ScaleTranslation(const TVector<T>& InScale3D)
{
	Translation *= InScale3D;

	DiagnosticCheckNaN_Translate();
}


template<typename T>
inline void TTransform<T>::ScaleTranslation(const FReal& Scale)
{
	Translation *= Scale;

	DiagnosticCheckNaN_Translate();
}


// this function is from matrix, and all it does is to normalize rotation portion
template<typename T>
inline void TTransform<T>::RemoveScaling(T Tolerance/*=SMALL_NUMBER*/)
{
	Scale3D = TVector<T>(1, 1, 1);
	Rotation.Normalize();

	DiagnosticCheckNaN_Rotate();
	DiagnosticCheckNaN_Scale3D();
}

template<typename T>
UE_FORCEINLINE_HINT void TTransform<T>::MultiplyUsingMatrixWithScale(TTransform<T>* OutTransform, const TTransform<T>* A, const TTransform<T>* B)
{
	// the goal of using M is to get the correct orientation
	// but for translation, we still need scale
	ConstructTransformFromMatrixWithDesiredScale(A->ToMatrixWithScale(), B->ToMatrixWithScale(), A->Scale3D*B->Scale3D, *OutTransform);
}

template<typename T>
inline void TTransform<T>::ConstructTransformFromMatrixWithDesiredScale(const TMatrix<T>& AMatrix, const TMatrix<T>& BMatrix, const TVector<T>& DesiredScale, TTransform<T>& OutTransform)
{
	// the goal of using M is to get the correct orientation
	// but for translation, we still need scale
	TMatrix<T> M = AMatrix * BMatrix;
	M.RemoveScaling();

	// apply negative scale back to axes
	TVector<T> SignedScale = DesiredScale.GetSignVector();

	M.SetAxis(0, SignedScale.X * M.GetScaledAxis(EAxis::X));
	M.SetAxis(1, SignedScale.Y * M.GetScaledAxis(EAxis::Y));
	M.SetAxis(2, SignedScale.Z * M.GetScaledAxis(EAxis::Z));

	// @note: if you have negative with 0 scale, this will return rotation that is identity
	// since matrix loses that axes
	TQuat<T> Rotation = TQuat<T>(M);
	Rotation.Normalize();

	// set values back to output
	OutTransform.Scale3D = DesiredScale;
	OutTransform.Rotation = Rotation;

	// technically I could calculate this using TTransform<T> but then it does more quat multiplication 
	// instead of using Scale in matrix multiplication
	// it's a question of between RemoveScaling vs using TTransform<T> to move translation
	OutTransform.Translation = M.GetOrigin();
}

/** Returns Multiplied Transform of 2 TTransform<T>s **/
template<typename T>
inline void TTransform<T>::Multiply(TTransform<T>* OutTransform, const TTransform<T>* A, const TTransform<T>* B)
{
	A->DiagnosticCheckNaN_All();
	B->DiagnosticCheckNaN_All();

	checkSlow(A->IsRotationNormalized());
	checkSlow(B->IsRotationNormalized());

	//	When Q = quaternion, S = single scalar scale, and T = translation
	//	QST(A) = Q(A), S(A), T(A), and QST(B) = Q(B), S(B), T(B)

	//	QST (AxB) 

	// QST(A) = Q(A)*S(A)*P*-Q(A) + T(A)
	// QST(AxB) = Q(B)*S(B)*QST(A)*-Q(B) + T(B)
	// QST(AxB) = Q(B)*S(B)*[Q(A)*S(A)*P*-Q(A) + T(A)]*-Q(B) + T(B)
	// QST(AxB) = Q(B)*S(B)*Q(A)*S(A)*P*-Q(A)*-Q(B) + Q(B)*S(B)*T(A)*-Q(B) + T(B)
	// QST(AxB) = [Q(B)*Q(A)]*[S(B)*S(A)]*P*-[Q(B)*Q(A)] + Q(B)*S(B)*T(A)*-Q(B) + T(B)

	//	Q(AxB) = Q(B)*Q(A)
	//	S(AxB) = S(A)*S(B)
	//	T(AxB) = Q(B)*S(B)*T(A)*-Q(B) + T(B)

	if (AnyHasNegativeScale(A->Scale3D, B->Scale3D))
	{
		// @note, if you have 0 scale with negative, you're going to lose rotation as it can't convert back to quat
		MultiplyUsingMatrixWithScale(OutTransform, A, B);
	}
	else
	{
		OutTransform->Rotation = B->Rotation*A->Rotation;
		OutTransform->Scale3D = A->Scale3D*B->Scale3D;
		OutTransform->Translation = B->Rotation*(B->Scale3D*A->Translation) + B->Translation;
	}

	// we do not support matrix transform when non-uniform
	// that was removed at rev 21 with UE4
}
/**
* Apply Scale to this transform
*/
template<typename T>
inline TTransform<T> TTransform<T>::GetScaled(T InScale) const
{
	TTransform<T> A(*this);
	A.Scale3D *= InScale;

	A.DiagnosticCheckNaN_Scale3D();

	return A;
}


/**
* Apply Scale to this transform
*/
template<typename T>
inline TTransform<T> TTransform<T>::GetScaled(TVector<T> InScale) const
{
	TTransform<T> A(*this);
	A.Scale3D *= InScale;

	A.DiagnosticCheckNaN_Scale3D();

	return A;
}


/** Transform homogenous TVector4<T>, ignoring the scaling part of this transform **/
template<typename T>
inline TVector4<T> TTransform<T>::TransformFVector4NoScale(const TVector4<T>& V) const
{
	DiagnosticCheckNaN_All();

	// if not, this won't work
	checkSlow(V.W == 0.f || V.W == 1.f);

	//Transform using QST is following
	//QST(P) = Q*S*P*-Q + T where Q = quaternion, S = scale, T = translation
	TVector4<T> Transform = TVector4<T>(Rotation.RotateVector(TVector<T>(V)), 0.f);
	if (V.W == 1.f)
	{
		Transform += TVector4<T>(Translation, 1.f);
	}

	return Transform;
}


/** Transform TVector4<T> **/
template<typename T>
inline TVector4<T> TTransform<T>::TransformFVector4(const TVector4<T>& V) const
{
	DiagnosticCheckNaN_All();

	// if not, this won't work
	checkSlow(V.W == 0.f || V.W == 1.f);

	//Transform using QST is following
	//QST(P) = Q*S*P*-Q + T where Q = quaternion, S = scale, T = translation

	TVector4<T> Transform = TVector4<T>(Rotation.RotateVector(Scale3D*TVector<T>(V)), 0.f);
	if (V.W == 1.f)
	{
		Transform += TVector4<T>(Translation, 1.f);
	}

	return Transform;
}


template<typename T>
inline TVector<T> TTransform<T>::TransformPosition(const TVector<T>& V) const
{
	DiagnosticCheckNaN_All();
	return Rotation.RotateVector(Scale3D*V) + Translation;
}


template<typename T>
inline TVector<T> TTransform<T>::TransformPositionNoScale(const TVector<T>& V) const
{
	DiagnosticCheckNaN_All();
	return Rotation.RotateVector(V) + Translation;
}


template<typename T>
inline TVector<T> TTransform<T>::TransformVector(const TVector<T>& V) const
{
	DiagnosticCheckNaN_All();
	return Rotation.RotateVector(Scale3D*V);
}


template<typename T>
inline TVector<T> TTransform<T>::TransformVectorNoScale(const TVector<T>& V) const
{
	DiagnosticCheckNaN_All();
	return Rotation.RotateVector(V);
}


// do backward operation when inverse, translation -> rotation -> scale
template<typename T>
inline TVector<T> TTransform<T>::InverseTransformPosition(const TVector<T> &V) const
{
	DiagnosticCheckNaN_All();
	return (Rotation.UnrotateVector(V - Translation)) * GetSafeScaleReciprocal(Scale3D);
}


// do backward operation when inverse, translation -> rotation
template<typename T>
inline TVector<T> TTransform<T>::InverseTransformPositionNoScale(const TVector<T> &V) const
{
	DiagnosticCheckNaN_All();
	return (Rotation.UnrotateVector(V - Translation));
}


// do backward operation when inverse, translation -> rotation -> scale
template<typename T>
inline TVector<T> TTransform<T>::InverseTransformVector(const TVector<T> &V) const
{
	DiagnosticCheckNaN_All();
	return (Rotation.UnrotateVector(V)) * GetSafeScaleReciprocal(Scale3D);
}


// do backward operation when inverse, translation -> rotation
template<typename T>
inline TVector<T> TTransform<T>::InverseTransformVectorNoScale(const TVector<T> &V) const
{
	DiagnosticCheckNaN_All();
	return (Rotation.UnrotateVector(V));
}

template<typename T>
UE_FORCEINLINE_HINT TQuat<T> TTransform<T>::TransformRotation(const TQuat<T>& Q) const
{
	return GetRotation() * Q;
}

template<typename T>
UE_FORCEINLINE_HINT TQuat<T> TTransform<T>::InverseTransformRotation(const TQuat<T>& Q) const
{
	return GetRotation().Inverse() * Q;
}

template<typename T>
inline TTransform<T> TTransform<T>::operator*(const TTransform<T>& Other) const
{
	TTransform<T> Output;
	Multiply(&Output, this, &Other);
	return Output;
}


template<typename T>
UE_FORCEINLINE_HINT void TTransform<T>::operator*=(const TTransform<T>& Other)
{
	Multiply(this, this, &Other);
}


template<typename T>
inline TTransform<T> TTransform<T>::operator*(const TQuat<T>& Other) const
{
	TTransform<T> Output, OtherTransform(Other, TVector<T>::ZeroVector, TVector<T>::OneVector);
	Multiply(&Output, this, &OtherTransform);
	return Output;
}


template<typename T>
inline void TTransform<T>::operator*=(const TQuat<T>& Other)
{
	TTransform<T> OtherTransform(Other, TVector<T>::ZeroVector, TVector<T>::OneVector);
	Multiply(this, this, &OtherTransform);
}


// x = 0, y = 1, z = 2
template<typename T>
inline TVector<T> TTransform<T>::GetScaledAxis(EAxis::Type InAxis) const
{
	if (InAxis == EAxis::X)
	{
		return TransformVector(TVector<T>(1.f, 0.f, 0.f));
	}
	else if (InAxis == EAxis::Y)
	{
		return TransformVector(TVector<T>(0.f, 1.f, 0.f));
	}

	return TransformVector(TVector<T>(0.f, 0.f, 1.f));
}


// x = 0, y = 1, z = 2
template<typename T>
inline TVector<T> TTransform<T>::GetUnitAxis(EAxis::Type InAxis) const
{
	if (InAxis == EAxis::X)
	{
		return TransformVectorNoScale(TVector<T>(1.f, 0.f, 0.f));
	}
	else if (InAxis == EAxis::Y)
	{
		return TransformVectorNoScale(TVector<T>(0.f, 1.f, 0.f));
	}

	return TransformVectorNoScale(TVector<T>(0.f, 0.f, 1.f));
}


template<typename T>
inline void TTransform<T>::Mirror(EAxis::Type MirrorAxis, EAxis::Type FlipAxis)
{
	// We do convert to Matrix for mirroring. 
	TMatrix<T> M = ToMatrixWithScale();
	M.Mirror(MirrorAxis, FlipAxis);
	SetFromMatrix(M);
}


/** same version of TMatrix<T>::GetMaximumAxisScale function **/
/** @return the maximum magnitude of all components of the 3D scale. */
template<typename T>
inline T TTransform<T>::GetMaximumAxisScale() const
{
	DiagnosticCheckNaN_Scale3D();
	return (float)Scale3D.GetAbsMax();
}


/** @return the minimum magnitude of all components of the 3D scale. */
template<typename T>
inline T TTransform<T>::GetMinimumAxisScale() const
{
	DiagnosticCheckNaN_Scale3D();
	return (float)Scale3D.GetAbsMin();
}


// mathematically if you have 0 scale, it should be infinite, 
// however, in practice if you have 0 scale, and relative transform doesn't make much sense 
// anymore because you should be instead of showing gigantic infinite mesh
// also returning BIG_NUMBER causes sequential NaN issues by multiplying 
// so we hardcode as 0
template<typename T>
inline TVector<T> TTransform<T>::GetSafeScaleReciprocal(const TVector<T>& InScale, T Tolerance)
{
	TVector<T> SafeReciprocalScale;
	if (FMath::Abs(InScale.X) <= Tolerance)
	{
		SafeReciprocalScale.X = 0.f;
	}
	else
	{
		SafeReciprocalScale.X = 1 / InScale.X;
	}

	if (FMath::Abs(InScale.Y) <= Tolerance)
	{
		SafeReciprocalScale.Y = 0.f;
	}
	else
	{
		SafeReciprocalScale.Y = 1 / InScale.Y;
	}

	if (FMath::Abs(InScale.Z) <= Tolerance)
	{
		SafeReciprocalScale.Z = 0.f;
	}
	else
	{
		SafeReciprocalScale.Z = 1 / InScale.Z;
	}

	return SafeReciprocalScale;
}

} // namespace UE::Math
} // namespace UE

#endif
