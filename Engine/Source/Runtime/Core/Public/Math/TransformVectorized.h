// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "Math/Axis.h"
#include "Math/MathFwd.h"
#include "Math/Matrix.h"
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Math/ScalarRegister.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Math/VectorRegister.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Build.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Templates/IsFloatingPoint.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/NameTypes.h"

#if ENABLE_VECTORIZED_TRANSFORM

struct Z_Construct_UScriptStruct_FTransform3d_Statics;
struct Z_Construct_UScriptStruct_FTransform3f_Statics;
struct Z_Construct_UScriptStruct_FTransform_Statics;

namespace UE
{
namespace Math
{
	// Selector for alignment of TTransform based on template type used.
	// The base has no 'Value' defined, you must define a specialized type below for types allowed as TTransform's template param.
	template<typename T>
	struct TAlignOfTransform {  };

	// Specializations for allowed TTransform::FReal types
	template<> struct TAlignOfTransform<float>		{ enum { Value = FMath::Max<uint64>(16, alignof(TPersistentVectorRegisterType<float>)) }; };
	template<> struct TAlignOfTransform<double>		{ enum { Value = FMath::Max<uint64>(16, alignof(TPersistentVectorRegisterType<double>)) }; };

/**
 * Transform composed of Scale, Rotation (as a quaternion), and Translation.
 *
 * Transforms can be used to convert from one space to another, for example by transforming
 * positions and directions from local space to world space.
 *
 * Transformation of position vectors is applied in the order:  Scale -> Rotate -> Translate.
 * Transformation of direction vectors is applied in the order: Scale -> Rotate.
 *
 * Order matters when composing transforms: C = A * B will yield a transform C that logically
 * first applies A then B to any subsequent transformation. Note that this is the opposite order of quaternion (TQuat<T>) multiplication.
 *
 * Example: LocalToWorld = (DeltaRotation * LocalToWorld) will change rotation in local space by DeltaRotation.
 * Example: LocalToWorld = (LocalToWorld * DeltaRotation) will change rotation in world space by DeltaRotation.
 */
template<typename T>
struct alignas( TAlignOfTransform<T>::Value ) TTransform
{
	// Can't have a UE_REQUIRES in the declaration because of the forward declarations, so check for allowed types here.
	static_assert(std::is_floating_point_v<T>, "TTransform only supports float and double types.");

	friend Z_Construct_UScriptStruct_FTransform3f_Statics;
	friend Z_Construct_UScriptStruct_FTransform3d_Statics;
	friend Z_Construct_UScriptStruct_FTransform_Statics;

	using FReal = T;
	using TransformVectorRegister = TVectorRegisterType<T>;

protected:
	/** Rotation of this transformation, as a quaternion */
	TPersistentVectorRegisterType<T> Rotation;
	/** Translation of this transformation, as a vector */
	TPersistentVectorRegisterType<T> Translation;
	/** 3D scale (always applied in local space) as a vector */
	TPersistentVectorRegisterType<T> Scale3D;
public:
	/**
	 * The identity transformation (Rotation = TQuat<T>::Identity, Translation = TVector<T>::ZeroVector, Scale3D = (1,1,1))
	 */
	CORE_API static const TTransform<T> Identity;

#if ENABLE_NAN_DIAGNOSTIC
	inline void DiagnosticCheckNaN_Scale3D() const
	{
		if (VectorContainsNaNOrInfinite(Scale3D))
		{
			logOrEnsureNanError(TEXT("TTransform<T> Vectorized Scale3D contains NaN"));
		}
	}

	inline void DiagnosticCheckNaN_Translate() const
	{
		if (VectorContainsNaNOrInfinite(Translation))
		{
			logOrEnsureNanError(TEXT("TTransform<T> Vectorized Translation contains NaN"));
		}
	}

	inline void DiagnosticCheckNaN_Rotate() const
	{
		if (VectorContainsNaNOrInfinite(Rotation))
		{
			logOrEnsureNanError(TEXT("TTransform<T> Vectorized Rotation contains NaN"));
		}
	}

	inline void DiagnosticCheckNaN_All() const
	{
		DiagnosticCheckNaN_Scale3D();
		DiagnosticCheckNaN_Rotate();
		DiagnosticCheckNaN_Translate();
	}

	inline void DiagnosticCheck_IsValid() const
	{
		DiagnosticCheckNaN_All();
		if (!IsValid())
		{
			logOrEnsureNanError(TEXT("TTransform Vectorized transform is not valid: %s"), *ToHumanReadableString());
		}
		
	}
#else
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN_Translate() const {}
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN_Rotate() const {}
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN_Scale3D() const {}
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN_All() const {}
	UE_FORCEINLINE_HINT void DiagnosticCheck_IsValid() const {}
#endif

	/**
	 * Constructor with initialization to the identity transform.
	 */
	[[nodiscard]] inline TTransform()
	{
		// Rotation = {0,0,0,1)
		Rotation = GlobalVectorConstants::Float0001;
		// Translation = {0,0,0,0)
		Translation = GlobalVectorConstants::FloatZero;
		// Scale3D = {1,1,1,0);
		Scale3D = GlobalVectorConstants::Float1110;
	}

	/**
	 * Constructor with an initial translation
	 *
	 * @param InTranslation The value to use for the translation component
	 */
	[[nodiscard]] inline explicit TTransform(const TVector<T>& InTranslation) 
	{
		// Rotation = {0,0,0,1) quaternion identity
		Rotation = GlobalVectorConstants::Float0001;
		//Translation = InTranslation;
		Translation = MakeVectorRegister(InTranslation.X, InTranslation.Y, InTranslation.Z, 0.0f );
		// Scale3D = {1,1,1,0);
		Scale3D = GlobalVectorConstants::Float1110;

		DiagnosticCheckNaN_All();
	}

	/**
	 * Constructor with an initial rotation
	 *
	 * @param InRotation The value to use for rotation component
	 */
	[[nodiscard]] inline explicit TTransform(const TQuat<T>& InRotation) 
	{
		// Rotation = InRotation
		Rotation =  VectorLoadAligned( &InRotation );
		// Translation = {0,0,0,0)
		Translation = GlobalVectorConstants::FloatZero;
		// Scale3D = {1,1,1,0);
		Scale3D = GlobalVectorConstants::Float1110;

		DiagnosticCheckNaN_All();
	}

	/**
	 * Constructor with an initial rotation
	 *
	 * @param InRotation The value to use for rotation component  (after being converted to a quaternion)
	 */
	[[nodiscard]] inline explicit TTransform(const TRotator<T>& InRotation) 
	{
		TQuat<T> InQuatRotation = TQuat<T>::MakeFromRotator(InRotation);
		// Rotation = InRotation
		Rotation =  VectorLoadAligned( &InQuatRotation );
		// Translation = {0,0,0,0)
		Translation = GlobalVectorConstants::FloatZero;
		// Scale3D = {1,1,1,0);
		Scale3D = GlobalVectorConstants::Float1110;

		DiagnosticCheckNaN_All();
	}

	/**
	 * Constructor with all components initialized
	 *
	 * @param InRotation The value to use for rotation component
	 * @param InTranslation The value to use for the translation component
	 * @param InScale3D The value to use for the scale component
	 */
	[[nodiscard]] inline TTransform(const TQuat<T>& InRotation, const TVector<T>& InTranslation, const TVector<T>& InScale3D = TVector<T>(1.f, 1.f, 1.f))
	{
		// Rotation = InRotation
		Rotation =  VectorLoadAligned( &InRotation );
		// Translation = InTranslation
		Translation = MakeVectorRegister(InTranslation.X, InTranslation.Y, InTranslation.Z, 0.0f );
		// Scale3D = InScale3D
		Scale3D = MakeVectorRegister(InScale3D.X, InScale3D.Y, InScale3D.Z, 0.0f );

		DiagnosticCheckNaN_All();
	}

	/**
	 * Constructor with all components initialized as VectorRegisters
	 *
	 * @param InRotation The value to use for rotation component
	 * @param InTranslation The value to use for the translation component
	 * @param InScale3D The value to use for the scale component
	 */
	[[nodiscard]] inline TTransform(TransformVectorRegister InRotation, TransformVectorRegister InTranslation, TransformVectorRegister InScale3D)
		: Rotation(InRotation),
		Translation(InTranslation),
		Scale3D(InScale3D)
	{
		DiagnosticCheckNaN_All();
	}

	/**
	 * Constructor with all components initialized, taking a FRotator as the rotation component
	 *
	 * @param InRotation The value to use for rotation component (after being converted to a quaternion)
	 * @param InTranslation The value to use for the translation component
	 * @param InScale3D The value to use for the scale component
	 */
	[[nodiscard]] inline TTransform(const TRotator<T>& InRotation, const TVector<T>& InTranslation, const TVector<T>& InScale3D = TVector<T>(1.f, 1.f, 1.f))
	{
		TQuat<T> InQuatRotation = TQuat<T>::MakeFromRotator(InRotation);
		// Rotation = InRotation
		Rotation =  VectorLoadAligned( &InQuatRotation );
		// Translation = InTranslation
		Translation = MakeVectorRegister(InTranslation.X, InTranslation.Y, InTranslation.Z, 0.0f );
		// Scale3D = InScale3D
		Scale3D = MakeVectorRegister(InScale3D.X, InScale3D.Y, InScale3D.Z, 0.0f );

		DiagnosticCheckNaN_All();
	}

	/**
	 * Constructor with leaving uninitialized memory
	 */
	[[nodiscard]] inline explicit TTransform(ENoInit) 
	{
		// Note: This can be used to track down initialization issues with bone transform arrays; but it will
		// cause issues with transient fields such as RootMotionDelta that get initialized to 0 by default
#if ENABLE_NAN_DIAGNOSTIC
		FReal qnan = FMath::Log2(-5.3f);
		check(FMath::IsNaN(qnan));
		Translation = MakeVectorRegister(qnan, qnan, qnan, qnan);
		Rotation = MakeVectorRegister(qnan, qnan, qnan, qnan);
		Scale3D = MakeVectorRegister(qnan, qnan, qnan, qnan);
#endif
	}

	/**
	 * Constructor for converting a Matrix (including scale) into a TTransform.
	 */
	[[nodiscard]] inline explicit TTransform(const TMatrix<T>& InMatrix)
	{
		SetFromMatrix(InMatrix);
		DiagnosticCheckNaN_All();
	}

	/** Constructor that takes basis axes and translation */
	[[nodiscard]] inline TTransform(const TVector<T>& InX, const TVector<T>& InY, const TVector<T>& InZ, const TVector<T>& InTranslation)
	{
		SetFromMatrix(TMatrix<T>(InX, InY, InZ, InTranslation));
		DiagnosticCheckNaN_All();
	}

	/**
	 * Does a debugf of the contents of this Transform.
	 */
	CORE_API void DebugPrint() const;

	/** Debug purpose only **/
	[[nodiscard]] bool DebugEqualMatrix(const TMatrix<T>& Matrix) const;

	/** Convert TTransform contents to a string */
	[[nodiscard]] CORE_API FString ToHumanReadableString() const;

	[[nodiscard]] CORE_API FString ToString() const;

	/** Acceptable form: "%f,%f,%f|%f,%f,%f|%f,%f,%f" */
	CORE_API bool InitFromString( const FString& InSourceString );


	[[nodiscard]] inline TMatrix<T> ToMatrixWithScale() const
	{

		TMatrix<T> OutMatrix;
		TransformVectorRegister DiagonalsXYZ;
		TransformVectorRegister Adds;
		TransformVectorRegister Subtracts;

		ToMatrixInternal( DiagonalsXYZ, Adds, Subtracts );
		const TransformVectorRegister DiagonalsXYZ_W0 = VectorSet_W0(DiagonalsXYZ);

		// OutMatrix.M[0][0] = (1.0f - (yy2 + zz2)) * Scale.X;    // Diagonal.X
		// OutMatrix.M[0][1] = (xy2 + wz2) * Scale.X;             // Adds.X
		// OutMatrix.M[0][2] = (xz2 - wy2) * Scale.X;             // Subtracts.Z
		// OutMatrix.M[0][3] = 0.0f;                              // DiagonalsXYZ_W0.W
		const TransformVectorRegister AddX_DC_DiagX_DC = VectorShuffle(Adds, DiagonalsXYZ_W0, 0, 0, 0, 0);
		const TransformVectorRegister SubZ_DC_DiagW_DC = VectorShuffle(Subtracts, DiagonalsXYZ_W0, 2, 0, 3, 0);
		const TransformVectorRegister Row0 = VectorShuffle(AddX_DC_DiagX_DC, SubZ_DC_DiagW_DC, 2, 0, 0, 2);

		// OutMatrix.M[1][0] = (xy2 - wz2) * Scale.Y;             // Subtracts.X
		// OutMatrix.M[1][1] = (1.0f - (xx2 + zz2)) * Scale.Y;    // Diagonal.Y
		// OutMatrix.M[1][2] = (yz2 + wx2) * Scale.Y;             // Adds.Y
		// OutMatrix.M[1][3] = 0.0f;                            // DiagonalsXYZ_W0.W
		const TransformVectorRegister SubX_DC_DiagY_DC = VectorShuffle(Subtracts, DiagonalsXYZ_W0, 0, 0, 1, 0);
		const TransformVectorRegister AddY_DC_DiagW_DC = VectorShuffle(Adds, DiagonalsXYZ_W0, 1, 0, 3, 0);
		const TransformVectorRegister Row1 = VectorShuffle(SubX_DC_DiagY_DC, AddY_DC_DiagW_DC, 0, 2, 0, 2);

		// OutMatrix.M[2][0] = (xz2 + wy2) * Scale.Z;             // Adds.Z
		// OutMatrix.M[2][1] = (yz2 - wx2) * Scale.Z;             // Subtracts.Y
		// OutMatrix.M[2][2] = (1.0f - (xx2 + yy2)) * Scale.Z;    // Diagonals.Z
		// OutMatrix.M[2][3] = 0.0f;                              // DiagonalsXYZ_W0.W
		const TransformVectorRegister AddZ_DC_SubY_DC = VectorShuffle(Adds, Subtracts, 2, 0, 1, 0);
		const TransformVectorRegister Row2 = VectorShuffle(AddZ_DC_SubY_DC, DiagonalsXYZ_W0, 0, 2, 2, 3);

		VectorStoreAligned(Row0, &(OutMatrix.M[0][0]));
		VectorStoreAligned(Row1, &(OutMatrix.M[1][0]));
		VectorStoreAligned(Row2, &(OutMatrix.M[2][0]));

		// OutMatrix.M[3][0] = Translation.X;
		// OutMatrix.M[3][1] = Translation.Y;
		// OutMatrix.M[3][2] = Translation.Z;
		// OutMatrix.M[3][3] = 1.0f;
		const TransformVectorRegister Row3 = VectorSet_W1(Translation);
		VectorStoreAligned(Row3, &(OutMatrix.M[3][0]));

		return OutMatrix;
	}

	/**
	* Convert this Transform to matrix with scaling and compute the inverse of that.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TMatrix<T> ToInverseMatrixWithScale() const
	{
		// todo: optimize
		return ToMatrixWithScale().Inverse();
	}

	/**
	* Convert this Transform to inverse.
	*/
	[[nodiscard]] inline TTransform Inverse() const
	{
		// Replacement of Inverse of TMatrix<T>
		if (VectorAnyGreaterThan(VectorAbs(Scale3D), GlobalVectorConstants::SmallNumber))
		{
			return InverseFast();
		}
		else
		{
			return TTransform<T>::Identity;
		}
	}

	/**
	* Convert this Transform to a transformation matrix, ignoring its scaling
	*/
	[[nodiscard]] inline TMatrix<T> ToMatrixNoScale() const
	{
		TMatrix<T> OutMatrix;
		TransformVectorRegister DiagonalsXYZ;
		TransformVectorRegister Adds;
		TransformVectorRegister Subtracts;

		ToMatrixInternalNoScale( DiagonalsXYZ, Adds, Subtracts );
		const TransformVectorRegister DiagonalsXYZ_W0 = VectorSet_W0(DiagonalsXYZ);

		// OutMatrix.M[0][0] = (1.0f - (yy2 + zz2));			// Diagonal.X
		// OutMatrix.M[0][1] = (xy2 + wz2);						// Adds.X
		// OutMatrix.M[0][2] = (xz2 - wy2);						// Subtracts.Z
		// OutMatrix.M[0][3] = 0.0f;                            // DiagonalsXYZ_W0.W
		const TransformVectorRegister AddX_DC_DiagX_DC = VectorShuffle(Adds, DiagonalsXYZ_W0, 0, 0, 0, 0);
		const TransformVectorRegister SubZ_DC_DiagW_DC = VectorShuffle(Subtracts, DiagonalsXYZ_W0, 2, 0, 3, 0);
		const TransformVectorRegister Row0 = VectorShuffle(AddX_DC_DiagX_DC, SubZ_DC_DiagW_DC, 2, 0, 0, 2);

		// OutMatrix.M[1][0] = (xy2 - wz2);			            // Subtracts.X
		// OutMatrix.M[1][1] = (1.0f - (xx2 + zz2));		    // Diagonal.Y
		// OutMatrix.M[1][2] = (yz2 + wx2);						// Adds.Y
		// OutMatrix.M[1][3] = 0.0f;                            // DiagonalsXYZ_W0.W
		const TransformVectorRegister SubX_DC_DiagY_DC = VectorShuffle(Subtracts, DiagonalsXYZ_W0, 0, 0, 1, 0);
		const TransformVectorRegister AddY_DC_DiagW_DC = VectorShuffle(Adds, DiagonalsXYZ_W0, 1, 0, 3, 0);
		const TransformVectorRegister Row1 = VectorShuffle(SubX_DC_DiagY_DC, AddY_DC_DiagW_DC, 0, 2, 0, 2);

		// OutMatrix.M[2][0] = (xz2 + wy2);						// Adds.Z
		// OutMatrix.M[2][1] = (yz2 - wx2);						// Subtracts.Y
		// OutMatrix.M[2][2] = (1.0f - (xx2 + yy2));		    // Diagonals.Z
		// OutMatrix.M[2][3] = 0.0f;                            // DiagonalsXYZ_W0.W
		const TransformVectorRegister AddZ_DC_SubY_DC = VectorShuffle(Adds, Subtracts, 2, 0, 1, 0);
		const TransformVectorRegister Row2 = VectorShuffle(AddZ_DC_SubY_DC, DiagonalsXYZ_W0, 0, 2, 2, 3);

		VectorStoreAligned(Row0, &(OutMatrix.M[0][0]));
		VectorStoreAligned(Row1, &(OutMatrix.M[1][0]));
		VectorStoreAligned(Row2, &(OutMatrix.M[2][0]));

		// OutMatrix.M[3][0] = Translation.X;
		// OutMatrix.M[3][1] = Translation.Y;
		// OutMatrix.M[3][2] = Translation.Z;
		// OutMatrix.M[3][3] = 1.0f;
		const TransformVectorRegister Row3 = VectorSet_W1(Translation);
		VectorStoreAligned(Row3, &(OutMatrix.M[3][0]));

		return OutMatrix;
	}

	/** Set this transform to the weighted blend of the supplied two transforms. */
	inline void Blend(const TTransform<T>& Atom1, const TTransform<T>& Atom2, float Alpha)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_EDITORONLY_DATA
		// Check that all bone atoms coming from animation are normalized
		check( Atom1.IsRotationNormalized() );
		check( Atom2.IsRotationNormalized() );
#endif

		if( FMath::Abs(Alpha) <= ZERO_ANIMWEIGHT_THRESH )
		{
			// if blend is all the way for child1, then just copy its bone atoms
			(*this) = Atom1;
		}
		else if( FMath::Abs(Alpha - 1.0f) <= ZERO_ANIMWEIGHT_THRESH )
		{
			// if blend is all the way for child2, then just copy its bone atoms
			(*this) = Atom2;
		}
		else
		{
			// Simple linear interpolation for translation and scale.			
			TransformVectorRegister BlendWeight = VectorLoadFloat1(&Alpha);

			Translation = FMath::Lerp<TransformVectorRegister>(Atom1.Translation, Atom2.Translation, BlendWeight);
			Scale3D = FMath::Lerp<TransformVectorRegister>(Atom1.Scale3D, Atom2.Scale3D, BlendWeight);

			TransformVectorRegister VRotation = VectorLerpQuat(Atom1.Rotation, Atom2.Rotation, BlendWeight);

			// ..and renormalize
			Rotation = VectorNormalizeQuaternion(VRotation);

			DiagnosticCheckNaN_All(); // MR
		}
	}

	/** Set this Transform to the weighted blend of it and the supplied Transform. */
	inline void BlendWith(const TTransform<T>& OtherAtom, float Alpha)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_EDITORONLY_DATA
		// Check that all bone atoms coming from animation are normalized
		check( IsRotationNormalized() );
		check( OtherAtom.IsRotationNormalized() );
#endif

		if( Alpha > ZERO_ANIMWEIGHT_THRESH )
		{
			if( Alpha >= 1.f - ZERO_ANIMWEIGHT_THRESH )
			{
				// if blend is all the way for child2, then just copy its bone atoms
				(*this) = OtherAtom;
			}
			else 
			{
				// Simple linear interpolation for translation and scale.				
				TransformVectorRegister BlendWeight = VectorLoadFloat1(&Alpha);
				Translation = FMath::Lerp<TransformVectorRegister>(Translation, OtherAtom.Translation, BlendWeight);
				
				Scale3D = FMath::Lerp<TransformVectorRegister>(Scale3D, OtherAtom.Scale3D, BlendWeight);
				
				TransformVectorRegister VRotation = VectorLerpQuat(Rotation, OtherAtom.Rotation, BlendWeight);
				
				// ..and renormalize
				Rotation = VectorNormalizeQuaternion(VRotation);

				DiagnosticCheckNaN_All(); 
			}
		}
	}


	/**
	 * Quaternion addition is wrong here. This is just a special case for linear interpolation.
	 * Use only within blends!!
	 * Rotation part is NOT normalized!!
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TTransform<T> operator+(const TTransform<T>& Atom) const
	{
		return TTransform<T>( VectorAdd(Rotation, Atom.Rotation), VectorAdd(Translation, Atom.Translation ), VectorAdd( Scale3D, Atom.Scale3D ) );
	}

	inline TTransform<T>& operator+=(const TTransform<T>& Atom)
	{ 
		Translation = VectorAdd(Translation, Atom.Translation);
		Rotation = VectorAdd(Rotation, Atom.Rotation);
		Scale3D = VectorAdd(Scale3D, Atom.Scale3D);

		return *this;
	}

	[[nodiscard]] inline TTransform<T> operator*(const ScalarRegister& MultScalar) const
	{		
		const TransformVectorRegister Mult = MultScalar.Value;
		return TTransform<T>( VectorMultiply(Rotation, Mult), VectorMultiply(Translation, Mult), VectorMultiply(Scale3D, Mult) );
	}

	inline TTransform<T>& operator*=(const ScalarRegister& MultScalar)
	{			
		const TransformVectorRegister Mult = MultScalar.Value;
		Translation= VectorMultiply(Translation, Mult);
		Rotation = VectorMultiply(Rotation, Mult);
		Scale3D = VectorMultiply(Scale3D, Mult);

		return *this;
	}

	[[nodiscard]] inline TTransform<T>	operator*(const TTransform<T>& Other) const;
	UE_FORCEINLINE_HINT void			operator*=(const TTransform<T>& Other);
	[[nodiscard]] inline TTransform<T>	operator*(const TQuat<T>& Other) const;
	inline void							operator*=(const TQuat<T>& Other);

	[[nodiscard]] inline static bool AnyHasNegativeScale(const TVector<T>& InScale3D, const TVector<T>& InOtherScale3D);
	inline void ScaleTranslation(const TVector<T>& InScale3D);
	inline void ScaleTranslation(const T& Scale);
	inline void RemoveScaling(T Tolerance=UE_SMALL_NUMBER);
	[[nodiscard]] inline float GetMaximumAxisScale() const;
	[[nodiscard]] inline float GetMinimumAxisScale() const;
	// Inverse does not work well with VQS format(in particular non-uniform), so removing it, but made two below functions to be used instead. 

	/*******************************************************************************************
	 * The below 2 functions are the ones to get delta transform and return TTransform<T> format that can be concatenated
	 * Inverse itself can't concatenate with VQS format(since VQS always transform from S->Q->T, where inverse happens from T(-1)->Q(-1)->S(-1))
	 * So these 2 provides ways to fix this
	 * GetRelativeTransform returns this*Other(-1) and parameter is Other(not Other(-1))
	 * GetRelativeTransformReverse returns this(-1)*Other, and parameter is Other. 
	 *******************************************************************************************/
	[[nodiscard]] CORE_API TTransform<T> GetRelativeTransform(const TTransform<T>& Other) const;
	[[nodiscard]] CORE_API TTransform<T> GetRelativeTransformReverse(const TTransform<T>& Other) const;
	/**
	 * Set current transform and the relative to ParentTransform.
	 * Equates to This = This->GetRelativeTransform(Parent), but saves the intermediate TTransform<T> storage and copy.
	 */
	CORE_API void		SetToRelativeTransform(const TTransform<T>& ParentTransform);

	[[nodiscard]] inline TVector4<T>	TransformFVector4(const TVector4<T>& V) const;
	[[nodiscard]] inline TVector4<T>	TransformFVector4NoScale(const TVector4<T>& V) const;
	[[nodiscard]] inline TVector<T>		TransformPosition(const TVector<T>& V) const;
	[[nodiscard]] inline TVector<T>		TransformPositionNoScale(const TVector<T>& V) const;


	/** Inverts the transform and then transforms V - correctly handles scaling in this transform. */
	[[nodiscard]] inline TVector<T>		InverseTransformPosition(const TVector<T> &V) const;
	[[nodiscard]] inline TVector<T>		InverseTransformPositionNoScale(const TVector<T> &V) const;
	[[nodiscard]] inline TVector<T>		TransformVector(const TVector<T>& V) const;
	[[nodiscard]] inline TVector<T>		TransformVectorNoScale(const TVector<T>& V) const;

	/** 
	 *	Transform a direction vector by the inverse of this matrix - will not take into account translation part.
	 *	If you want to transform a surface normal (or plane) and correctly account for non-uniform scaling you should use TransformByUsingAdjointT with adjoint of matrix inverse.
	 */
	[[nodiscard]] inline TVector<T> InverseTransformVector(const TVector<T> &V) const;
	[[nodiscard]] inline TVector<T> InverseTransformVectorNoScale(const TVector<T> &V) const;

	/**
	* Transform a rotation.
	* For example if this is a LocalToWorld transform, TransformRotation(Q) would transform Q from local to world space.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TQuat<T> TransformRotation(const TQuat<T>& Q) const;

	/**
	* Inverse transform a rotation.
	* For example if this is a LocalToWorld transform, InverseTransformRotation(Q) would transform Q from world to local space.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TQuat<T> InverseTransformRotation(const TQuat<T>& Q) const;

	[[nodiscard]] inline TTransform<T>	GetScaled(T Scale) const;
	[[nodiscard]] inline TTransform<T>	GetScaled(TVector<T> Scale) const;
	[[nodiscard]] inline TVector<T>		GetScaledAxis(EAxis::Type InAxis) const;
	[[nodiscard]] inline TVector<T>		GetUnitAxis(EAxis::Type InAxis) const;
	inline void		Mirror(EAxis::Type MirrorAxis, EAxis::Type FlipAxis);
	[[nodiscard]] inline static TVector<T>	GetSafeScaleReciprocal(const TVector<T>& InScale, T Tolerance=UE_SMALL_NUMBER);


	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> GetLocation() const
	{
		return GetTranslation();
	}

	[[nodiscard]] inline TRotator<T> Rotator() const
	{
		TQuat<T> OutRotation;
		VectorStoreAligned(Rotation, &OutRotation);
		return OutRotation.Rotator();
	}

	/** Calculate the determinant */
	[[nodiscard]] inline T GetDeterminant() const
	{
		//#todo - vectorized version of this
		UE::Math::TVector4<T> OutScale3D;
		VectorStoreAligned(Scale3D, &OutScale3D);
		return OutScale3D.X * OutScale3D.Y * OutScale3D.Z;
	}

	/** Set the translation of this transformation */
	inline void SetLocation(const TVector<T>& Origin)
	{		
		Translation = VectorLoadFloat3_W0(&Origin);
		DiagnosticCheckNaN_Translate();
	}

	/**
	 * Checks the components for NaN's
	 * @return Returns true if any component (rotation, translation, or scale) is a NAN
	 */
	[[nodiscard]] bool ContainsNaN() const
	{
		if (VectorContainsNaNOrInfinite(Rotation))
		{
			return true;
		}
		if (VectorContainsNaNOrInfinite(Translation))
		{
			return true;
		}

		if (VectorContainsNaNOrInfinite(Scale3D))
		{
			return true;
		}
		return false;
	}

	[[nodiscard]] inline bool IsValid() const
	{
		if ( ContainsNaN() )
		{
			return false;
		}

		if ( !IsRotationNormalized() )
		{
			return false;
		}

		return true;
	}

	// Serializer.
	inline friend FArchive& operator<<(FArchive& Ar,TTransform<T>& M)
	{
		Ar << *reinterpret_cast<TQuat<T>*>(&(M.Rotation));
		Ar << *reinterpret_cast<TVector<T>*>(&(M.Translation));
		Ar << *reinterpret_cast<TVector<T>*>(&(M.Scale3D));
		
		if (Ar.IsLoading())
		{
			M.Translation = VectorSet_W0(M.Translation);
			M.Scale3D = VectorSet_W0(M.Scale3D);
		}
		return Ar;
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}
	
	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		// FTransform3f/d are immutable, but FTransform is not and needs a per-property conversion
		if(StructTag == NAME_Transform)
		{
			return false;
		}

		if constexpr (std::is_same_v<T, float>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Transform, Transform3f, Transform3d);
		}
		else
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Transform, Transform3d, Transform3f);
		}
	}
	
	// Binary comparison operators.
	/*
	bool operator==( const TTransform<T>& Other ) const
	{
		return Rotation==Other.Rotation && Translation==Other.Translation && Scale3D==Other.Scale3D;
	}
	bool operator!=( const TTransform<T>& Other ) const
	{
		return Rotation!=Other.Rotation || Translation!=Other.Translation || Scale3D!=Other.Scale3D;
	}
	*/

private:

	UE_FORCEINLINE_HINT static bool Private_AnyHasNegativeScale(TransformVectorRegister InScale3D, TransformVectorRegister InOtherScale3D)
	{
		return !!VectorAnyLesserThan(VectorMin(InScale3D, InOtherScale3D), (TransformVectorRegister)GlobalVectorConstants::FloatZero);
	}

	inline bool Private_RotationEquals(TransformVectorRegister InRotation, const FReal ToleranceScalar = UE_KINDA_SMALL_NUMBER) const
	{			
		const TransformVectorRegister MyRotation = Rotation; // Load from persistent value, to avoid repeated loads.
		const TransformVectorRegister Tolerance = VectorLoadFloat1(&ToleranceScalar);
		// !( (FMath::Abs(X-Q.X) > Tolerance) || (FMath::Abs(Y-Q.Y) > Tolerance) || (FMath::Abs(Z-Q.Z) > Tolerance) || (FMath::Abs(W-Q.W) > Tolerance) )
		const TransformVectorRegister RotationSub = VectorAbs(VectorSubtract(MyRotation, InRotation));
		// !( (FMath::Abs(X+Q.X) > Tolerance) || (FMath::Abs(Y+Q.Y) > Tolerance) || (FMath::Abs(Z+Q.Z) > Tolerance) || (FMath::Abs(W+Q.W) > Tolerance) )
		const TransformVectorRegister RotationAdd = VectorAbs(VectorAdd(MyRotation, InRotation));
		return !VectorAnyGreaterThan(RotationSub, Tolerance) || !VectorAnyGreaterThan(RotationAdd, Tolerance);
	}

	inline bool Private_TranslationEquals(TransformVectorRegister InTranslation, const FReal ToleranceScalar = UE_KINDA_SMALL_NUMBER) const
	{			
		const TransformVectorRegister Tolerance = VectorLoadFloat1(&ToleranceScalar);
		// !( (FMath::Abs(X-V.X) > Tolerance) || (FMath::Abs(Y-V.Y) > Tolerance) || (FMath::Abs(Z-V.Z) > Tolerance) )
		const TransformVectorRegister TranslationDiff = VectorAbs(VectorSubtract(Translation, InTranslation));		
		return !VectorAnyGreaterThan(TranslationDiff, Tolerance);
	}

	inline bool Private_Scale3DEquals(TransformVectorRegister InScale3D, const FReal ToleranceScalar = UE_KINDA_SMALL_NUMBER) const
	{
		const TransformVectorRegister Tolerance = VectorLoadFloat1(&ToleranceScalar);
		// !( (FMath::Abs(X-V.X) > Tolerance) || (FMath::Abs(Y-V.Y) > Tolerance) || (FMath::Abs(Z-V.Z) > Tolerance) )
		const TransformVectorRegister ScaleDiff = VectorAbs(VectorSubtract(Scale3D, InScale3D));
		return !VectorAnyGreaterThan(ScaleDiff, Tolerance);
	}

public:

	// Test if A's rotation equals B's rotation, within a tolerance. Preferred over "A.GetRotation().Equals(B.GetRotation())" because it is faster on some platforms.
	[[nodiscard]] UE_FORCEINLINE_HINT static bool AreRotationsEqual(const TTransform<T>& A, const TTransform<T>& B, FReal Tolerance=UE_KINDA_SMALL_NUMBER)
	{
		return A.Private_RotationEquals(B.Rotation, Tolerance);
	}

	// Test if A's translation equals B's translation, within a tolerance. Preferred over "A.GetTranslation().Equals(B.GetTranslation())" because it avoids TransformVectorRegister->TVector<T> conversion.
	[[nodiscard]] UE_FORCEINLINE_HINT static bool AreTranslationsEqual(const TTransform<T>& A, const TTransform<T>& B, FReal Tolerance=UE_KINDA_SMALL_NUMBER)
	{
		return A.Private_TranslationEquals(B.Translation, Tolerance);
	}

	// Test if A's scale equals B's scale, within a tolerance. Preferred over "A.GetScale3D().Equals(B.GetScale3D())" because it avoids TransformVectorRegister->TVector<T> conversion.
	[[nodiscard]] UE_FORCEINLINE_HINT static bool AreScale3DsEqual(const TTransform<T>& A, const TTransform<T>& B, FReal Tolerance=UE_KINDA_SMALL_NUMBER)
	{
		return A.Private_Scale3DEquals(B.Scale3D, Tolerance);
	}


	// Test if this Transform's rotation equals another's rotation, within a tolerance. Preferred over "GetRotation().Equals(Other.GetRotation())" because it is faster on some platforms.
	[[nodiscard]] UE_FORCEINLINE_HINT bool RotationEquals(const TTransform<T>& Other, FReal Tolerance=UE_KINDA_SMALL_NUMBER) const
	{
		return AreRotationsEqual(*this, Other, Tolerance);
	}

	// Test if this Transform's translation equals another's translation, within a tolerance. Preferred over "GetTranslation().Equals(Other.GetTranslation())" because it avoids TransformVectorRegister->TVector<T> conversion.
	[[nodiscard]] UE_FORCEINLINE_HINT bool TranslationEquals(const TTransform<T>& Other, FReal Tolerance=UE_KINDA_SMALL_NUMBER) const
	{
		return AreTranslationsEqual(*this, Other, Tolerance);
	}

	// Test if this Transform's scale equals another's scale, within a tolerance. Preferred over "GetScale3D().Equals(Other.GetScale3D())" because it avoids TransformVectorRegister->TVector<T> conversion.
	[[nodiscard]] UE_FORCEINLINE_HINT bool Scale3DEquals(const TTransform<T>& Other, FReal Tolerance=UE_KINDA_SMALL_NUMBER) const
	{
		return AreScale3DsEqual(*this, Other, Tolerance);
	}

	// Test if all components of the transforms are equal, within a tolerance.
	[[nodiscard]] UE_FORCEINLINE_HINT bool Equals(const TTransform<T>& Other, FReal Tolerance=UE_KINDA_SMALL_NUMBER) const
	{
		return Private_TranslationEquals(Other.Translation, Tolerance) && Private_RotationEquals(Other.Rotation, Tolerance) && Private_Scale3DEquals(Other.Scale3D, Tolerance);
	}

	// Test if all components of the transform property are equal.
	[[nodiscard]] UE_FORCEINLINE_HINT bool Identical(const TTransform<T>* Other, uint32 PortFlags) const
	{
		return Equals(*Other, 0.f);
	}

	// Test if rotation and translation components of the transforms are equal, within a tolerance.
	[[nodiscard]] UE_FORCEINLINE_HINT bool EqualsNoScale(const TTransform<T>& Other, FReal Tolerance=UE_KINDA_SMALL_NUMBER) const
	{
		return Private_TranslationEquals(Other.Translation, Tolerance) && Private_RotationEquals(Other.Rotation, Tolerance);
	}

	/** Returns Multiplied Transform of 2 FTransforms **/
	CORE_API static void Multiply(TTransform<T>* OutTransform, const TTransform<T>* A, const TTransform<T>* B);

	/**
	 * Sets the components
	 * @param InRotation The new value for the Rotation component
	 * @param InTranslation The new value for the Translation component
	 * @param InScale3D The new value for the Scale3D component
	 */
	inline void SetComponents(const TQuat<T>& InRotation, const TVector<T>& InTranslation, const TVector<T>& InScale3D) 
	{
		Rotation = VectorLoadAligned(&InRotation);
		Translation = VectorLoadFloat3_W0(&InTranslation);
		Scale3D = VectorLoadFloat3_W0(&InScale3D);

		DiagnosticCheckNaN_All();
	}

	/**
	 * Sets the components to the identity transform:
	 *   Rotation = (0,0,0,1)
	 *   Translation = (0,0,0)
	 *   Scale3D = (1,1,1)
	 */
	inline void SetIdentity()
	{
		// Rotation = {0,0,0,1)
		Rotation = GlobalVectorConstants::Float0001; //VectorSet_W1( VectorZero() );
		// Translation = {0,0,0,0)
		Translation = GlobalVectorConstants::FloatZero;
		// Scale3D = {1,1,1,0);
		Scale3D = GlobalVectorConstants::Float1110; //VectorSet_W0( VectorOne() );
	}

	/**
	 * Sets the components to the 'additive' identity transform:
	 *   Rotation = (0,0,0,1)
	 *   Translation = (0,0,0)
	 *   Scale3D = (0,0,0)
	 */
	inline void SetIdentityZeroScale()
	{
		// Rotation = {0,0,0,1)
		Rotation = GlobalVectorConstants::Float0001; //VectorSet_W1( VectorZero() );
		// Translation = {0,0,0,0)
		Translation = GlobalVectorConstants::FloatZero;
		// Scale3D = {0,0,0,0);
		Scale3D = GlobalVectorConstants::FloatZero;
	}
	
	/**
	 * Scales the Scale3D component by a new factor
	 * @param Scale3DMultiplier The value to multiply Scale3D with
	 */
	inline void MultiplyScale3D(const TVector<T>& Scale3DMultiplier)
	{
		Scale3D = VectorMultiply(Scale3D, VectorLoadFloat3_W0(&Scale3DMultiplier));
		DiagnosticCheckNaN_Scale3D();
	}

	/**
	 * Sets the translation component
	 * @param NewTranslation The new value for the translation component
	 */
	inline void SetTranslation(const TVector<T>& NewTranslation)
	{
		Translation = VectorLoadFloat3_W0(&NewTranslation);
		DiagnosticCheckNaN_Translate();
	}

	/** Copy translation from another TTransform<T>. */
	UE_FORCEINLINE_HINT void CopyTranslation(const TTransform<T>& Other)
	{
		Translation = Other.Translation;
	}

	/**
	 * Concatenates another rotation to this transformation 
	 * @param DeltaRotation The rotation to concatenate in the following fashion: Rotation = Rotation * DeltaRotation
	 */
	inline void ConcatenateRotation(const TQuat<T>& DeltaRotation)
	{		
		Rotation = VectorQuaternionMultiply2(Rotation, VectorLoadAligned(&DeltaRotation));		
		DiagnosticCheckNaN_Rotate();
	}

	/**
	 * Adjusts the translation component of this transformation 
	 * @param DeltaTranslation The translation to add in the following fashion: Translation += DeltaTranslation
	 */
	inline void AddToTranslation(const TVector<T>& DeltaTranslation)
	{		
		Translation = VectorAdd(Translation, VectorLoadFloat3_W0(&DeltaTranslation));
		DiagnosticCheckNaN_Translate();
	}

	/**
	 * Add the translations from two FTransforms and return the result.
	 * @return A.Translation + B.Translation
	 */
	[[nodiscard]] inline static TVector<T> AddTranslations(const TTransform<T>& A, const TTransform<T>& B)
	{
		TVector<T> Result;
		VectorStoreFloat3(VectorAdd(A.Translation, B.Translation), &Result);
		return Result;
	}

	/**
	 * Subtract translations from two FTransforms and return the difference.
	 * @return A.Translation - B.Translation.
	 */
	[[nodiscard]] inline static TVector<T> SubtractTranslations(const TTransform<T>& A, const TTransform<T>& B)
	{
		TVector<T> Result;
		VectorStoreFloat3(VectorSubtract(A.Translation, B.Translation), &Result);
		return Result;
	}

	/**
	 * Sets the rotation component
	 * @param NewRotation The new value for the rotation component
	 */
	inline void SetRotation(const TQuat<T>& NewRotation)
	{
		Rotation = VectorLoadAligned(&NewRotation);
		DiagnosticCheckNaN_Rotate();
	}

	/** Copy rotation from another TTransform<T>. */
	UE_FORCEINLINE_HINT void CopyRotation(const TTransform<T>& Other)
	{
		Rotation = Other.Rotation;
	}

	/**
	 * Sets the Scale3D component
	 * @param NewScale3D The new value for the Scale3D component
	 */
	inline void SetScale3D(const TVector<T>& NewScale3D)
	{
		Scale3D = VectorLoadFloat3_W0(&NewScale3D);
		DiagnosticCheckNaN_Scale3D();
	}

	/** Copy scale from another TTransform<T>. */
	UE_FORCEINLINE_HINT void CopyScale3D(const TTransform<T>& Other)
	{
		Scale3D = Other.Scale3D;
	}

	/**
	 * Sets both the translation and Scale3D components at the same time
	 * @param NewTranslation The new value for the translation component
	 * @param NewScale3D The new value for the Scale3D component
	 */
	inline void SetTranslationAndScale3D(const TVector<T>& NewTranslation, const TVector<T>& NewScale3D)
	{
		Translation = VectorLoadFloat3_W0(&NewTranslation);
		Scale3D = VectorLoadFloat3_W0(&NewScale3D);

		DiagnosticCheckNaN_Translate();
		DiagnosticCheckNaN_Scale3D();
	}
	
	// For low-level VectorRegister programming
	[[nodiscard]] TPersistentVectorRegisterType<T> GetTranslationRegister() const
	{
		return Translation;
	}
	[[nodiscard]] TPersistentVectorRegisterType<T> GetRotationRegister() const
	{
		return Rotation;
	}
	void SetTranslationRegister(TransformVectorRegister InTranslation)
	{
		Translation = InTranslation;
	}
	void SetRotationRegister(TransformVectorRegister InRotation) 
	{
		Rotation = InRotation;
	}

	/** @note : Added template type function for Accumulate
	  * The template type isn't much useful yet, but it is with the plan to move forward
	  * to unify blending features with just type of additive or full pose
	  * Eventually it would be nice to just call blend and it all works depending on full pose
	  * or additive, but right now that is a lot more refactoring
	  * For now this types only defines the different functionality of accumulate
	  */

	/**
	* Accumulates another transform with this one
	*
	* Rotation is accumulated multiplicatively (Rotation = SourceAtom.Rotation * Rotation)
	* Translation is accumulated additively (Translation += SourceAtom.Translation)
	* Scale3D is accumulated multiplicatively (Scale3D *= SourceAtom.Scale3D)
	*
	* @param SourceAtom The other transform to accumulate into this one
	*/
	inline void Accumulate(const TTransform<T>& SourceAtom)
	{
		// Rotation = SourceAtom.Rotation * Rotation;
		Rotation = VectorQuaternionMultiply2(SourceAtom.Rotation, Rotation);

		// Translation += SourceAtom.Translation;
		// Scale *= SourceAtom.Scale;
		Translation = VectorAdd(Translation, SourceAtom.Translation);
		Scale3D = VectorMultiply(Scale3D, SourceAtom.Scale3D);

		DiagnosticCheckNaN_All();

		checkSlow(IsRotationNormalized());
	}

	/**
	* Accumulates another transform with this one, with a blending weight
	*
	* Let SourceAtom = Atom * BlendWeight
	* Rotation is accumulated multiplicatively (Rotation = SourceAtom.Rotation * Rotation).
	* Translation is accumulated additively (Translation += SourceAtom.Translation)
	* Scale3D is accumulated multiplicatively (Scale3D *= SourceAtom.Scale3D)
	*
	* Note: Rotation will not be normalized! Will have to be done manually.
	*
	* @param Atom The other transform to accumulate into this one
	* @param BlendWeight The weight to multiply Atom by before it is accumulated.
	*/
	inline void Accumulate(const TTransform<T>& Atom, const ScalarRegister& BlendWeightScalar)
	{
		// SourceAtom = Atom * BlendWeight;
		const TransformVectorRegister BlendWeight(BlendWeightScalar.Value);
		const TransformVectorRegister BlendedRotation = VectorMultiply(Atom.Rotation, BlendWeight);
		const TransformVectorRegister BlendedScale = VectorMultiply(Atom.Scale3D, BlendWeight);

		// Rotation = SourceAtom.Rotation * Rotation;
		Rotation = VectorQuaternionMultiply2(BlendedRotation, Rotation);

		// Translation += SourceAtom.Translation;
		// Scale *= SourceAtom.Scale;
		Translation = VectorMultiplyAdd(Atom.Translation, BlendWeight, Translation);
		Scale3D = VectorMultiply(Scale3D, BlendedScale);

		DiagnosticCheckNaN_All();
	}
	/**
	 * Accumulates another transform with this one, with an optional blending weight
	 *
	 * Rotation is accumulated additively, in the shortest direction (Rotation = Rotation +/- DeltaAtom.Rotation * Weight)
	 * Translation is accumulated additively (Translation += DeltaAtom.Translation * Weight)
	 * Scale3D is accumulated additively (Scale3D += DeltaAtom.Scale * Weight)
	 *
	 * @param DeltaAtom The other transform to accumulate into this one
	 * @param Weight The weight to multiply DeltaAtom by before it is accumulated.
	 */
	inline void AccumulateWithShortestRotation(const TTransform<T>& DeltaAtom, const ScalarRegister& BlendWeightScalar)
	{
		const TransformVectorRegister BlendWeight(BlendWeightScalar.Value);
		const TransformVectorRegister BlendedRotation = VectorMultiply(DeltaAtom.Rotation, BlendWeight);

		Rotation = VectorAccumulateQuaternionShortestPath(Rotation, BlendedRotation);

		Translation = VectorMultiplyAdd(DeltaAtom.Translation, BlendWeight, Translation);
		Scale3D = VectorMultiplyAdd(DeltaAtom.Scale3D, BlendWeight, Scale3D);

		DiagnosticCheckNaN_All();
	}

	/** Accumulates another transform with this one, with a blending weight
	*
	* Let SourceAtom = Atom * BlendWeight
	* Rotation is accumulated multiplicatively (Rotation = SourceAtom.Rotation * Rotation).
	* Translation is accumulated additively (Translation += SourceAtom.Translation)
	* Scale3D is accumulated assuming incoming scale is additive scale (Scale3D *= (1 + SourceAtom.Scale3D))
	*
	* When we create additive, we create additive scale based on [TargetScale/SourceScale -1]
	* because that way when you apply weight of 0.3, you don't shrink. We only saves the % of grow/shrink
	* when we apply that back to it, we add back the 1, so that it goes back to it.
	* This solves issue where you blend two additives with 0.3, you don't come back to 0.6 scale, but 1 scale at the end
	* because [1 + [1-1]*0.3 + [1-1]*0.3] becomes 1, so you don't shrink by applying additive scale
	*
	* Note: Rotation will not be normalized! Will have to be done manually.
	*
	* @param Atom The other transform to accumulate into this one
	* @param BlendWeight The weight to multiply Atom by before it is accumulated.
	*/
	inline void AccumulateWithAdditiveScale(const TTransform<T>& Atom, const ScalarRegister& BlendWeightScalar)
	{
		const TransformVectorRegister BlendWeight(BlendWeightScalar.Value);
		const TransformVectorRegister DefaultScale = GlobalVectorConstants::Float1110;

		// SourceAtom = Atom * BlendWeight;
		const TransformVectorRegister BlendedRotation = VectorMultiply(Atom.Rotation, BlendWeight);

		// Rotation = SourceAtom.Rotation * Rotation;
		Rotation = VectorQuaternionMultiply2(BlendedRotation, Rotation);

		// Translation += SourceAtom.Translation;
		// Scale *= SourceAtom.Scale;
		Translation = VectorMultiplyAdd(Atom.Translation, BlendWeight, Translation);
		Scale3D = VectorMultiply(Scale3D, VectorMultiplyAdd(Atom.Scale3D, BlendWeight, DefaultScale));


		DiagnosticCheckNaN_All();
	}

	/**
	 * Set the translation and Scale3D components of this transform to a linearly interpolated combination of two other transforms
	 *
	 * Translation = FMath::Lerp(SourceAtom1.Translation, SourceAtom2.Translation, Alpha)
	 * Scale3D = FMath::Lerp(SourceAtom1.Scale3D, SourceAtom2.Scale3D, Alpha)
	 *
	 * @param SourceAtom1 The starting point source atom (used 100% if Alpha is 0)
	 * @param SourceAtom2 The ending point source atom (used 100% if Alpha is 1)
	 * @param Alpha The blending weight between SourceAtom1 and SourceAtom2
	 */
	inline void LerpTranslationScale3D(const TTransform<T>& SourceAtom1, const TTransform<T>& SourceAtom2, const ScalarRegister& AlphaScalar)
	{
		const TransformVectorRegister Alpha(AlphaScalar.Value);
		Translation	= FMath::Lerp<TransformVectorRegister>(SourceAtom1.Translation, SourceAtom2.Translation, Alpha);
		Scale3D = FMath::Lerp<TransformVectorRegister>(SourceAtom1.Scale3D, SourceAtom2.Scale3D, Alpha);

		DiagnosticCheckNaN_Translate();
		DiagnosticCheckNaN_Scale3D();
	}

	/**
	 * Normalize the rotation component of this transformation
	 */
	inline void NormalizeRotation()
	{
		Rotation = VectorNormalizeQuaternion(Rotation);
		DiagnosticCheckNaN_Rotate();
	}

	/**
	 * Checks whether the rotation component is normalized or not
	 *
	 * @return true if the rotation component is normalized, and false otherwise.
	 */
	[[nodiscard]] inline bool IsRotationNormalized() const
	{		
		const TransformVectorRegister TestValue = VectorAbs(VectorSubtract(VectorOne(), VectorDot4(Rotation, Rotation)));
		return !VectorAnyGreaterThan(TestValue, GlobalVectorConstants::ThreshQuatNormalized);
	}

	/**
	 * Blends the Identity transform with a weighted source transform and accumulates that into a destination transform
	 *
	 * SourceAtom = Blend(Identity, SourceAtom, BlendWeight)
	 * FinalAtom.Rotation = SourceAtom.Rotation * FinalAtom.Rotation
	 * FinalAtom.Translation += SourceAtom.Translation
	 * FinalAtom.Scale3D *= SourceAtom.Scale3D
	 *
	 * @param FinalAtom [in/out] The atom to accumulate the blended source atom into
	 * @param SourceAtom The target transformation (used when BlendWeight = 1)
	 * @param Alpha The blend weight between Identity and SourceAtom
	 */
	inline static void BlendFromIdentityAndAccumulate(TTransform<T>& FinalAtom, const TTransform<T>& SourceAtom, const ScalarRegister& BlendWeightScalar)
	{
		const TransformVectorRegister BlendWeight(BlendWeightScalar.Value);
		const TransformVectorRegister Const0001 = GlobalVectorConstants::Float0001;
		const TransformVectorRegister ConstNegative0001 = VectorSubtract(VectorZero(), Const0001);
		const TransformVectorRegister VOneMinusAlpha = VectorSubtract(VectorOne(), BlendWeight);
		const TransformVectorRegister DefaultScale = GlobalVectorConstants::Float1110;

		// Blend rotation
		//     To ensure the 'shortest route', we make sure the dot product between the both rotations is positive.
		//     const float Bias = (|A.B| >= 0 ? 1 : -1)
		//     BlendedAtom.Rotation = (B * Alpha) + (A * (Bias * (1.f - Alpha)));
		//     BlendedAtom.Rotation.QuaternionNormalize();
		//  Note: A = (0,0,0,1), which simplifies things a lot; only care about sign of B.W now, instead of doing a dot product
		const TransformVectorRegister RotationB = SourceAtom.Rotation;

		const TransformVectorRegister QuatRotationDirMask = VectorCompareGE(RotationB, VectorZero());
		const TransformVectorRegister BiasTimesA = VectorSelect(QuatRotationDirMask, Const0001, ConstNegative0001);
		const TransformVectorRegister RotateBTimesWeight = VectorMultiply(RotationB, BlendWeight);
		const TransformVectorRegister UnnormalizedRotation = VectorMultiplyAdd(BiasTimesA, VOneMinusAlpha, RotateBTimesWeight);

		// Normalize blended rotation ( result = (Q.Q >= 1e-8) ? (Q / |Q|) : (0,0,0,1) )
		const TransformVectorRegister BlendedRotation = VectorNormalizeSafe(UnnormalizedRotation, Const0001);

		// FinalAtom.Rotation = BlendedAtom.Rotation * FinalAtom.Rotation;
		FinalAtom.Rotation = VectorQuaternionMultiply2(BlendedRotation, FinalAtom.Rotation);

		// Blend translation and scale
		//    BlendedAtom.Translation = Lerp(Zero, SourceAtom.Translation, Alpha);
		//    BlendedAtom.Scale = Lerp(0, SourceAtom.Scale, Alpha);
		const TransformVectorRegister Zero = VectorZero();
		const TransformVectorRegister BlendedTranslation = FMath::Lerp<TransformVectorRegister>(Zero, SourceAtom.Translation, BlendWeight);
		const TransformVectorRegister BlendedScale3D = FMath::Lerp<TransformVectorRegister>(Zero, SourceAtom.Scale3D, BlendWeight);

		// Apply translation and scale to final atom
		//     FinalAtom.Translation += BlendedAtom.Translation
		//     FinalAtom.Scale *= BlendedAtom.Scale
		FinalAtom.Translation = VectorAdd( FinalAtom.Translation, BlendedTranslation );
		FinalAtom.Scale3D = VectorMultiply( FinalAtom.Scale3D, VectorAdd(DefaultScale, BlendedScale3D));
		checkSlow( FinalAtom.IsRotationNormalized() );
	}


	/**
	 * Returns the rotation component
	 *
	 * @return The rotation component
	 */
	[[nodiscard]] inline TQuat<T> GetRotation() const
	{
		DiagnosticCheckNaN_Rotate();
		TQuat<T> OutRotation;
		VectorStoreAligned(Rotation, &OutRotation);
		return OutRotation;
	}

	/**
	 * Returns the translation component
	 *
	 * @return The translation component
	 */
	[[nodiscard]] inline TVector<T> GetTranslation() const
	{
		DiagnosticCheckNaN_Translate();
		TVector<T> OutTranslation;
		VectorStoreFloat3(Translation, &OutTranslation);
		return OutTranslation;
	}

	/**
	 * Returns the Scale3D component
	 *
	 * @return The Scale3D component
	 */
	[[nodiscard]] inline TVector<T> GetScale3D() const
	{
		DiagnosticCheckNaN_Scale3D();
		TVector<T> OutScale3D;
		VectorStoreFloat3(Scale3D, &OutScale3D);
		return OutScale3D;
	}

	/**
	 * Sets the Rotation and Scale3D of this transformation from another transform
	 *
	 * @param SrcBA The transform to copy rotation and Scale3D from
	 */
	inline void CopyRotationPart(const TTransform<T>& SrcBA)
	{
		Rotation = SrcBA.Rotation;
		Scale3D = SrcBA.Scale3D;

		DiagnosticCheckNaN_Rotate();
		DiagnosticCheckNaN_Scale3D();
	}

	/**
	 * Sets the Translation and Scale3D of this transformation from another transform
	 *
	 * @param SrcBA The transform to copy translation and Scale3D from
	 */
	inline void CopyTranslationAndScale3D(const TTransform<T>& SrcBA)
	{
		Translation = SrcBA.Translation;
		Scale3D = SrcBA.Scale3D;

		DiagnosticCheckNaN_Translate();
		DiagnosticCheckNaN_Scale3D();
	}

	void SetFromMatrix(const TMatrix<T>& InMatrix)
	{
		TMatrix<T> M = InMatrix;

		// Get the 3D scale from the matrix
		TVector<T> InScale = M.ExtractScaling(0);
		Scale3D = VectorLoadFloat3_W0(&InScale);

		// If there is negative scaling going on, we handle that here
		if(InMatrix.Determinant() < 0.f)
		{
			// Assume it is along X and modify transform accordingly. 
			// It doesn't actually matter which axis we choose, the 'appearance' will be the same
			Scale3D = VectorMultiply(Scale3D, (TransformVectorRegister)GlobalVectorConstants::FloatMinus1_111 );
			M.SetAxis(0, -M.GetScaledAxis( EAxis::X ));
		}

		TQuat<T> InRotation = TQuat<T>(M);
		Rotation = VectorLoadAligned(&InRotation);
		TVector<T> InTranslation = InMatrix.GetOrigin();
		Translation = VectorLoadFloat3_W0(&InTranslation);

		// Normalize rotation
		Rotation = VectorNormalizeQuaternion(Rotation);		
	}

private:
	inline void ToMatrixInternal(TransformVectorRegister& OutDiagonals, TransformVectorRegister& OutAdds, TransformVectorRegister& OutSubtracts) const
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_EDITORONLY_DATA
		// Make sure Rotation is normalized when we turn it into a matrix.
		check( IsRotationNormalized() );
#endif

		TransformVectorRegister One;
		if constexpr (std::is_same_v<T, double>)
		{
			One = VectorOneDouble();
		}
		else
		{
			One = VectorOne();
		}

		const TransformVectorRegister MyRotation = Rotation; // Load from persistent value, to avoid repeated loads.
		const TransformVectorRegister RotationX2Y2Z2 = VectorAdd(MyRotation, MyRotation);	// x2, y2, z2
		const TransformVectorRegister RotationXX2YY2ZZ2 = VectorMultiply(RotationX2Y2Z2, MyRotation);	// xx2, yy2, zz2		

		// The diagonal terms of the rotation matrix are:
		//   (1 - (yy2 + zz2)) * scale
		//   (1 - (xx2 + zz2)) * scale
		//   (1 - (xx2 + yy2)) * scale
		const TransformVectorRegister yy2_xx2_xx2 = VectorSwizzle(RotationXX2YY2ZZ2, 1, 0, 0, 0);
		const TransformVectorRegister zz2_zz2_yy2 = VectorSwizzle(RotationXX2YY2ZZ2, 2, 2, 1, 0);
		const TransformVectorRegister DiagonalSum = VectorAdd(yy2_xx2_xx2, zz2_zz2_yy2);
		const TransformVectorRegister Diagonals = VectorSubtract(One, DiagonalSum);
		OutDiagonals = VectorMultiply(Diagonals, Scale3D);

		// Grouping the non-diagonal elements in the rotation block by operations:
		//    ((x*y2,y*z2,x*z2) + (w*z2,w*x2,w*y2)) * scale.xyz and
		//    ((x*y2,y*z2,x*z2) - (w*z2,w*x2,w*y2)) * scale.yxz
		// Rearranging so the LHS and RHS are in the same order as for +
		//    ((x*y2,y*z2,x*z2) - (w*z2,w*x2,w*y2)) * scale.yxz

		// RotBase = x*y2, y*z2, x*z2
		// RotOffset = w*z2, w*x2, w*y2
		const TransformVectorRegister x_y_x = VectorSwizzle(MyRotation, 0, 1, 0, 0);
		const TransformVectorRegister y2_z2_z2 = VectorSwizzle(RotationX2Y2Z2, 1, 2, 2, 0);
		const TransformVectorRegister RotBase = VectorMultiply(x_y_x, y2_z2_z2);

		const TransformVectorRegister w_w_w = VectorReplicate(MyRotation, 3);
		const TransformVectorRegister z2_x2_y2 = VectorSwizzle(RotationX2Y2Z2, 2, 0, 1, 0);
		const TransformVectorRegister RotOffset = VectorMultiply(w_w_w, z2_x2_y2);

		// Adds = (RotBase + RotOffset)*Scale3D :  (x*y2 + w*z2) * Scale3D.X , (y*z2 + w*x2) * Scale3D.Y, (x*z2 + w*y2) * Scale3D.Z
		// Subtracts = (RotBase - RotOffset)*Scale3DYZX :  (x*y2 - w*z2) * Scale3D.Y , (y*z2 - w*x2) * Scale3D.Z, (x*z2 - w*y2) * Scale3D.X
		const TransformVectorRegister Adds = VectorAdd(RotBase, RotOffset);
		OutAdds = VectorMultiply(Adds, Scale3D);
		const TransformVectorRegister Scale3DYZXW = VectorSwizzle( Scale3D, 1, 2, 0, 3);
		const TransformVectorRegister Subtracts = VectorSubtract(RotBase, RotOffset);
		OutSubtracts = VectorMultiply(Subtracts , Scale3DYZXW);
	}

	inline void ToMatrixInternalNoScale(TransformVectorRegister& OutDiagonals, TransformVectorRegister& OutAdds, TransformVectorRegister& OutSubtracts) const
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_EDITORONLY_DATA
		// Make sure Rotation is normalized when we turn it into a matrix.
		ensure( IsRotationNormalized() );
#endif		
		const TransformVectorRegister MyRotation = Rotation; // Load from persistent value, to avoid repeated loads.
		const TransformVectorRegister RotationX2Y2Z2 = VectorAdd(MyRotation, MyRotation);	// x2, y2, z2
		const TransformVectorRegister RotationXX2YY2ZZ2 = VectorMultiply(RotationX2Y2Z2, MyRotation);	// xx2, yy2, zz2		

		// The diagonal terms of the rotation matrix are:
		//   (1 - (yy2 + zz2))
		//   (1 - (xx2 + zz2))
		//   (1 - (xx2 + yy2))
		const TransformVectorRegister yy2_xx2_xx2 = VectorSwizzle(RotationXX2YY2ZZ2, 1, 0, 0, 0);
		const TransformVectorRegister zz2_zz2_yy2 = VectorSwizzle(RotationXX2YY2ZZ2, 2, 2, 1, 0);
		const TransformVectorRegister DiagonalSum = VectorAdd(yy2_xx2_xx2, zz2_zz2_yy2);
		OutDiagonals = VectorSubtract(VectorOne(), DiagonalSum);

		// Grouping the non-diagonal elements in the rotation block by operations:
		//    ((x*y2,y*z2,x*z2) + (w*z2,w*x2,w*y2)) and
		//    ((x*y2,y*z2,x*z2) - (w*z2,w*x2,w*y2))
		// Rearranging so the LHS and RHS are in the same order as for +
		//    ((x*y2,y*z2,x*z2) - (w*z2,w*x2,w*y2))

		// RotBase = x*y2, y*z2, x*z2
		// RotOffset = w*z2, w*x2, w*y2
		const TransformVectorRegister x_y_x = VectorSwizzle(MyRotation, 0, 1, 0, 0);
		const TransformVectorRegister y2_z2_z2 = VectorSwizzle(RotationX2Y2Z2, 1, 2, 2, 0);
		const TransformVectorRegister RotBase = VectorMultiply(x_y_x, y2_z2_z2);

		const TransformVectorRegister w_w_w = VectorReplicate(MyRotation, 3);
		const TransformVectorRegister z2_x2_y2 = VectorSwizzle(RotationX2Y2Z2, 2, 0, 1, 0);
		const TransformVectorRegister RotOffset = VectorMultiply(w_w_w, z2_x2_y2);

		// Adds = (RotBase + RotOffset):  (x*y2 + w*z2) , (y*z2 + w*x2), (x*z2 + w*y2)
		// Subtracts = (RotBase - RotOffset) :  (x*y2 - w*z2) , (y*z2 - w*x2), (x*z2 - w*y2)
		OutAdds = VectorAdd(RotBase, RotOffset);		
		OutSubtracts = VectorSubtract(RotBase, RotOffset);
	}

	/** 
	 * mathematically if you have 0 scale, it should be infinite, 
	 * however, in practice if you have 0 scale, and relative transform doesn't make much sense 
	 * anymore because you should be instead of showing gigantic infinite mesh
	 * also returning BIG_NUMBER causes sequential NaN issues by multiplying 
	 * so we hardcode as 0
	 */
	[[nodiscard]] static inline TransformVectorRegister	GetSafeScaleReciprocal(TransformVectorRegister InScale, const ScalarRegister& Tolerance = ScalarRegister(GlobalVectorConstants::SmallNumber))
	{		
		// SafeReciprocalScale.X = (InScale.X == 0) ? 0.f : 1/InScale.X; // same for YZW

		/// TransformVectorRegister( 1.0f / InScale.x, 1.0f / InScale.y, 1.0f / InScale.z, 1.0f / InScale.w )
		const TransformVectorRegister ReciprocalScale = VectorReciprocalAccurate(InScale);
		
		//TransformVectorRegister( Vec1.x == Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
		const TransformVectorRegister ScaleZeroMask = VectorCompareGE(TransformVectorRegister(Tolerance.Value), VectorAbs(InScale));

		//const TransformVectorRegister ScaleZeroMask = VectorCompareEQ(InScale, VectorZero());

		// TransformVectorRegister( for each bit i: Mask[i] ? Vec1[i] : Vec2[i] )
		const TransformVectorRegister SafeReciprocalScale = VectorSelect(ScaleZeroMask, VectorZero(), ReciprocalScale);

		return SafeReciprocalScale;
	}

	/** Returns Inverse Transform of this TTransform<T> **/
	[[nodiscard]] inline TTransform<T> InverseFast() const
	{
		// Inverse QST (A) = QST (~A)
		// Since A*~A = Identity, 
		// A(P) = Q(A)*S(A)*P*-Q(A) + T(A)
		// ~A(A(P)) = Q(~A)*S(~A)*(Q(A)*S(A)*P*-Q(A) + T(A))*-Q(~A) + T(~A) = Identity
		// Q(~A)*Q(A)*S(~A)*S(A)*P*-Q(A)*-Q(~A) + Q(~A)*S(~A)*T(A)*-Q(~A) + T(~A) = Identity
		// [Q(~A)*Q(A)]*[S(~A)*S(A)]*P*-[Q(~A)*Q(A)] + [Q(~A)*S(~A)*T(A)*-Q(~A) + T(~A)] = I

		// Identity Q = (0, 0, 0, 1) = Q(~A)*Q(A)
		// Identity Scale = 1 = S(~A)*S(A)
		// Identity Translation = (0, 0, 0) = [Q(~A)*S(~A)*T(A)*-Q(~A) + T(~A)]

		//	Q(~A) = Q(~A)
		//	S(~A) = 1.f/S(A)
		//	T(~A) = - (Q(~A)*S(~A)*T(A)*Q(A))	
		checkSlow(IsRotationNormalized());
		checkSlow(VectorAnyGreaterThan(VectorAbs(Scale3D), GlobalVectorConstants::SmallNumber));

		// Invert the scale
		const TransformVectorRegister InvScale = VectorSet_W0(GetSafeScaleReciprocal(VectorSet_W1(Scale3D), ScalarRegister(GlobalVectorConstants::SmallNumber)));

		// Invert the rotation
		const TransformVectorRegister InvRotation = VectorQuaternionInverse(Rotation);

		// Invert the translation
		const TransformVectorRegister ScaledTranslation = VectorMultiply(InvScale, Translation);
		const TransformVectorRegister t2 = VectorQuaternionRotateVector(InvRotation, ScaledTranslation);
		const TransformVectorRegister InvTranslation = VectorSet_W0(VectorNegate(t2));

		return TTransform<T>(InvRotation, InvTranslation, InvScale);
	}

	/**
	* Create a new transform: OutTransform = A * B using the matrix while keeping the scale that's given by A and B
	* Please note that this operation is a lot more expensive than normal Multiply
	*
	* Order matters when composing transforms : A * B will yield a transform that logically first applies A then B to any subsequent transformation.
	*
	* @param  OutTransform pointer to transform that will store the result of A * B.
	* @param  A Transform A.
	* @param  B Transform B.
	*/
	UE_FORCEINLINE_HINT static void MultiplyUsingMatrixWithScale(TTransform<T>* OutTransform, const TTransform<T>* A, const TTransform<T>* B);
	/**
	* Create a new transform from multiplications of given to matrices (AMatrix*BMatrix) using desired scale
	* This is used by MultiplyUsingMatrixWithScale and GetRelativeTransformUsingMatrixWithScale
	* This is only used to handle negative scale
	*
	* @param	AMatrix first Matrix of operation
	* @param	BMatrix second Matrix of operation
	* @param	DesiredScale - there is no check on if the magnitude is correct here. It assumes that is correct. 
	* @param	OutTransform the constructed transform 
	*/
	inline static void ConstructTransformFromMatrixWithDesiredScale(const TMatrix<T>& AMatrix, const TMatrix<T>& BMatrix, TransformVectorRegister DesiredScale, TTransform<T>& OutTransform);
	/**
	* Create a new transform: OutTransform = Base * Relative(-1) using the matrix while keeping the scale that's given by Base and Relative
	* Please note that this operation is a lot more expensive than normal GetRelativeTrnasform
	*
	* @param  OutTransform pointer to transform that will store the result of Base * Relative(-1).
	* @param  BAse Transform Base.
	* @param  Relative Transform Relative.
	*/
	static void GetRelativeTransformUsingMatrixWithScale(TTransform<T>* OutTransform, const TTransform<T>* Base, const TTransform<T>* Relative);

public:

	// Conversion to other type.
	friend struct TTransform<double>;
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg> && std::is_same_v<T, float>)>
	explicit TTransform(const TTransform<FArg>& From)
		: TTransform(MakeVectorRegisterFloatFromDouble(From.Rotation), MakeVectorRegisterFloatFromDouble(From.Translation), MakeVectorRegisterFloatFromDouble(From.Scale3D))
	{
	}

	friend struct TTransform<float>;
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg> && std::is_same_v<T, double>)>
	explicit TTransform(const TTransform<FArg>& From)
		: TTransform(MakeVectorRegisterDouble(From.Rotation), MakeVectorRegisterDouble(From.Translation), MakeVectorRegisterDouble(From.Scale3D))
	{
	}
};

#if !defined(_MSC_VER) || defined(__clang__)  // MSVC can't forward declare explicit specializations
template<> CORE_API const FTransform3f FTransform3f::Identity;
template<> CORE_API const FTransform3d FTransform3d::Identity;
#endif


template<typename T>
inline bool TTransform<T>::AnyHasNegativeScale(const TVector<T>& InScale3D, const TVector<T>& InOtherScale3D)
{
	TransformVectorRegister VectorInScale3D = VectorLoadFloat3_W0(&InScale3D);
	TransformVectorRegister VectorInOtherScale3D = VectorLoadFloat3_W0(&InOtherScale3D);

	return Private_AnyHasNegativeScale(VectorInScale3D, VectorInOtherScale3D);
}

/** Scale the translation part of the Transform by the supplied vector. */
template<typename T>
inline void TTransform<T>::ScaleTranslation(const TVector<T>& InScale3D)
{
	TransformVectorRegister VectorInScale3D = VectorLoadFloat3_W0(&InScale3D);
	Translation = VectorMultiply( Translation, VectorInScale3D );
	DiagnosticCheckNaN_Translate();
}

/** Scale the translation part of the Transform by the supplied value. */
template<typename T>
UE_FORCEINLINE_HINT void TTransform<T>::ScaleTranslation(const T& InScale)
{
	ScaleTranslation( TVector<T>(InScale) );
}

// this function is from matrix, and all it does is to normalize rotation portion
template<typename T>
inline void TTransform<T>::RemoveScaling(T Tolerance/*=UE_SMALL_NUMBER*/)
{
	Scale3D = GlobalVectorConstants::Float1110;
	NormalizeRotation();	

	DiagnosticCheckNaN_Rotate();
	DiagnosticCheckNaN_Scale3D();
}

template<typename T>
UE_FORCEINLINE_HINT void TTransform<T>::MultiplyUsingMatrixWithScale(TTransform<T>* OutTransform, const TTransform<T>* A, const TTransform<T>* B)
{
	ConstructTransformFromMatrixWithDesiredScale(A->ToMatrixWithScale(), B->ToMatrixWithScale(), VectorMultiply(A->Scale3D, B->Scale3D), *OutTransform);
}

template<typename T>
inline void TTransform<T>::ConstructTransformFromMatrixWithDesiredScale(const UE::Math::TMatrix<T>& AMatrix, const UE::Math::TMatrix<T>& BMatrix, TransformVectorRegister DesiredScale, TTransform<T>& OutTransform)
{
	// the goal of using M is to get the correct orientation
	// but for translation, we still need scale
	UE::Math::TMatrix<T> M = AMatrix * BMatrix;
	M.RemoveScaling();

	// apply negative scale back to axes
	TVector<T> SignedScale;
	VectorStoreFloat3(VectorSign(DesiredScale), &SignedScale);

	M.SetAxis(0, SignedScale.X * M.GetScaledAxis(EAxis::X));
	M.SetAxis(1, SignedScale.Y * M.GetScaledAxis(EAxis::Y));
	M.SetAxis(2, SignedScale.Z * M.GetScaledAxis(EAxis::Z));

	// @note: if you have negative with 0 scale, this will return rotation that is identity
	// since matrix loses that axes
	TQuat<T> Rotation = TQuat<T>(M);
	Rotation.Normalize();

	// set values back to output
	OutTransform.Scale3D = DesiredScale;
	OutTransform.Rotation = VectorLoadAligned(&Rotation);

	// technically I could calculate this using TTransform<T> but then it does more quat multiplication 
	// instead of using Scale in matrix multiplication
	// it's a question of between RemoveScaling vs using TTransform<T> to move translation
	TVector<T> Translation = M.GetOrigin();
	OutTransform.Translation = VectorLoadFloat3_W0(&Translation);
}

/** 
 * Apply Scale to this transform
 */
template<typename T>
inline TTransform<T> TTransform<T>::GetScaled(T InScale) const
{
	TTransform<T> A(*this);
	
	TransformVectorRegister VScale = VectorLoadFloat1(&InScale);
	A.Scale3D = VectorMultiply( A.Scale3D, VScale);

	A.DiagnosticCheckNaN_Scale3D();

	return A;
}

/** 
 * Apply Scale to this transform
 */
template<typename T>
inline TTransform<T> TTransform<T>::GetScaled(TVector<T> InScale) const
{
	TTransform<T> A(*this);

	TransformVectorRegister VScale = VectorLoadFloat3_W0(&InScale);
	A.Scale3D = VectorMultiply( A.Scale3D, VScale);

	A.DiagnosticCheckNaN_Scale3D();

	return A;
}

template<typename T>
inline TVector4<T> TTransform<T>::TransformFVector4NoScale(const TVector4<T>& V) const
{
	DiagnosticCheckNaN_All();

	// if not, this won't work
	checkSlow (V.W == 0.f || V.W == 1.f);

	const TransformVectorRegister InputVector = VectorLoadAligned(&V);

	//Transform using QST is following
	//QST(P) = Q.Rotate(S*P) + T where Q = quaternion, S = 1.0f, T = translation

	//RotatedVec = Q.Rotate(V.X, V.Y, V.Z, 0.f)
	const TransformVectorRegister InputVectorW0 = VectorSet_W0(InputVector);	
	const TransformVectorRegister RotatedVec = VectorQuaternionRotateVector(Rotation, InputVectorW0);

	// NewVect.XYZ += Translation * W
	// NewVect.W += 1 * W
	const TransformVectorRegister WWWW = VectorReplicate(InputVector, 3);
	const TransformVectorRegister TranslatedVec = VectorMultiplyAdd(VectorSet_W1(Translation), WWWW, RotatedVec);

	TVector4<T> NewVectOutput;
	VectorStoreAligned(TranslatedVec, &NewVectOutput);
	return NewVectOutput;
}

template<typename T>
inline TVector4<T> TTransform<T>::TransformFVector4(const TVector4<T>& V) const
{
	DiagnosticCheckNaN_All();

	// if not, this won't work
	checkSlow (V.W == 0.f || V.W == 1.f);

	const TransformVectorRegister InputVector = VectorLoadAligned(&V);

	//Transform using QST is following
	//QST(P) = Q.Rotate(S*P) + T where Q = quaternion, S = scale, T = translation

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const TransformVectorRegister InputVectorW0 = VectorSet_W0(InputVector);
	const TransformVectorRegister ScaledVec = VectorMultiply(Scale3D, InputVectorW0);
	const TransformVectorRegister RotatedVec = VectorQuaternionRotateVector(Rotation, ScaledVec);

	// NewVect.XYZ += Translation * W
	// NewVect.W += 1 * W
	const TransformVectorRegister WWWW = VectorReplicate(InputVector, 3);
	const TransformVectorRegister TranslatedVec = VectorMultiplyAdd(VectorSet_W1(Translation), WWWW, RotatedVec);

	TVector4<T> NewVectOutput;
	VectorStoreAligned(TranslatedVec, &NewVectOutput);
	return NewVectOutput;
}


template<typename T>
inline TVector<T> TTransform<T>::TransformPosition(const TVector<T>& V) const
{
	DiagnosticCheckNaN_All();

	const TransformVectorRegister InputVectorW0 = VectorLoadFloat3_W0(&V);

	//Transform using QST is following
	//QST(P) = Q.Rotate(S*P) + T where Q = quaternion, S = scale, T = translation
	
	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const TransformVectorRegister ScaledVec = VectorMultiply(Scale3D, InputVectorW0);
	const TransformVectorRegister RotatedVec = VectorQuaternionRotateVector(Rotation, ScaledVec);

	const TransformVectorRegister TranslatedVec = VectorAdd(RotatedVec, Translation);

	TVector<T> Result;
	VectorStoreFloat3(TranslatedVec, &Result);
	return Result;
}

template<typename T>
inline TVector<T> TTransform<T>::TransformPositionNoScale(const TVector<T>& V) const
{
	DiagnosticCheckNaN_All();

	const TransformVectorRegister InputVectorW0 = VectorLoadFloat3_W0(&V);

	//Transform using QST is following
	//QST(P) = Q.Rotate(S*P) + T where Q = quaternion, S = 1.0f, T = translation

	//RotatedVec = Q.Rotate(V.X, V.Y, V.Z, 0.f)
	const TransformVectorRegister RotatedVec = VectorQuaternionRotateVector(Rotation, InputVectorW0);

	const TransformVectorRegister TranslatedVec = VectorAdd(RotatedVec, Translation);

	TVector<T> Result;
	VectorStoreFloat3(TranslatedVec, &Result);
	return Result;
}

template<typename T>
inline TVector<T> TTransform<T>::TransformVector(const TVector<T>& V) const
{
	DiagnosticCheckNaN_All();

	const TransformVectorRegister InputVectorW0 = VectorLoadFloat3_W0(&V);

	//RotatedVec = Q.Rotate(Scale*V.X, Scale*V.Y, Scale*V.Z, 0.f)
	const TransformVectorRegister ScaledVec = VectorMultiply(Scale3D, InputVectorW0);
	const TransformVectorRegister RotatedVec = VectorQuaternionRotateVector(Rotation, ScaledVec);

	TVector<T> Result;
	VectorStoreFloat3(RotatedVec, &Result);
	return Result;
}

template<typename T>
inline TVector<T> TTransform<T>::TransformVectorNoScale(const TVector<T>& V) const
{
	DiagnosticCheckNaN_All();

	const TransformVectorRegister InputVectorW0 = VectorLoadFloat3_W0(&V);

	//RotatedVec = Q.Rotate(V.X, V.Y, V.Z, 0.f)
	const TransformVectorRegister RotatedVec = VectorQuaternionRotateVector(Rotation, InputVectorW0);

	TVector<T> Result;
	VectorStoreFloat3(RotatedVec, &Result);
	return Result;
}

// do backward operation when inverse, translation -> rotation -> scale
template<typename T>
inline TVector<T> TTransform<T>::InverseTransformPosition(const TVector<T> &V) const
{
	DiagnosticCheckNaN_All();

	const TransformVectorRegister InputVector = VectorLoadFloat3_W0(&V);

	// (V-Translation)
	const TransformVectorRegister TranslatedVec = VectorSet_W0(VectorSubtract(InputVector, Translation));

	// ( Rotation.Inverse() * (V-Translation) )
	const TransformVectorRegister VR = VectorQuaternionInverseRotateVector(Rotation, TranslatedVec);

	// GetSafeScaleReciprocal(Scale3D);
	const TransformVectorRegister SafeReciprocal = GetSafeScaleReciprocal(VectorSet_W1(Scale3D));	

	// ( Rotation.Inverse() * (V-Translation) ) * GetSafeScaleReciprocal(Scale3D);
	const TransformVectorRegister VResult = VectorMultiply(VR, SafeReciprocal);

	TVector<T> Result;
	VectorStoreFloat3(VResult, &Result);
	return Result;
}

// do backward operation when inverse, translation -> rotation
template<typename T>
inline TVector<T> TTransform<T>::InverseTransformPositionNoScale(const TVector<T> &V) const
{
	DiagnosticCheckNaN_All();

	const TransformVectorRegister InputVector = VectorLoadFloat3_W0(&V);

	// (V-Translation)
	const TransformVectorRegister TranslatedVec = VectorSet_W0(VectorSubtract(InputVector, Translation));

	// ( Rotation.Inverse() * (V-Translation) )
	const TransformVectorRegister VResult = VectorQuaternionInverseRotateVector(Rotation, TranslatedVec);

	TVector<T> Result;
	VectorStoreFloat3(VResult, &Result);
	return Result;
}


// do backward operation when inverse, translation -> rotation -> scale
template<typename T>
inline TVector<T> TTransform<T>::InverseTransformVector(const TVector<T> &V) const
{
	DiagnosticCheckNaN_All();

	const TransformVectorRegister InputVector = VectorLoadFloat3_W0(&V);

	// ( Rotation.Inverse() * V ) aka. TVector<T> TQuat<T>::operator*( const TVector<T>& V ) const
	const TransformVectorRegister VR = VectorQuaternionInverseRotateVector(Rotation, InputVector);

	// GetSafeScaleReciprocal(Scale3D);
	const TransformVectorRegister SafeReciprocal = GetSafeScaleReciprocal(VectorSet_W1(Scale3D));

	// ( Rotation.Inverse() * V) * GetSafeScaleReciprocal(Scale3D);
	const TransformVectorRegister VResult = VectorMultiply(VR, SafeReciprocal);

	TVector<T> Result;
	VectorStoreFloat3(VResult, &Result);
	return Result;
}

// do backward operation when inverse, translation -> rotation
template<typename T>
inline TVector<T> TTransform<T>::InverseTransformVectorNoScale(const TVector<T> &V) const
{
	DiagnosticCheckNaN_All();

	TransformVectorRegister InputVector = VectorLoadFloat3_W0(&V);

	// ( Rotation.Inverse() * V )
	TransformVectorRegister VResult = VectorQuaternionInverseRotateVector(Rotation, InputVector);

	TVector<T> Result;
	VectorStoreFloat3(VResult, &Result);
	return Result;
}

template<typename T>
UE_FORCEINLINE_HINT TQuat<T> TTransform<T>::TransformRotation(const TQuat<T>& Q) const
{
	return GetRotation() * Q;
}

template<typename T>
UE_FORCEINLINE_HINT TQuat<T> TTransform<T>::InverseTransformRotation(const TQuat<T>& Q) const
{
	return GetRotation().Inverse() * Q;
}

template<typename T>
inline TTransform<T> TTransform<T>::operator*(const TTransform<T>& Other) const
{
	TTransform<T> Output;
	Multiply(&Output, this, &Other);
	return Output;
}

template<typename T>
UE_FORCEINLINE_HINT void TTransform<T>::operator*=(const TTransform<T>& Other)
{
	Multiply(this, this, &Other);
}

template<typename T>
inline TTransform<T> TTransform<T>::operator*(const TQuat<T>& Other) const
{
	TTransform<T> Output, OtherTransform(Other, TVector<T>::ZeroVector, TVector<T>::OneVector);
	Multiply(&Output, this, &OtherTransform);
	return Output;
}

template<typename T>
inline void TTransform<T>::operator*=(const TQuat<T>& Other)
{
	TTransform<T> OtherTransform(Other, TVector<T>::ZeroVector, TVector<T>::OneVector);
	Multiply(this, this, &OtherTransform);
}

// x = 0, y = 1, z = 2
template<typename T>
inline TVector<T> TTransform<T>::GetScaledAxis(EAxis::Type InAxis) const
{
	if ( InAxis == EAxis::X )
	{
		return TransformVector(TVector<T>(1.f, 0.f, 0.f));
	}
	else if ( InAxis == EAxis::Y )
	{
		return TransformVector(TVector<T>(0.f, 1.f, 0.f));
	}

	return TransformVector(TVector<T>(0.f, 0.f, 1.f));
}

// x = 0, y = 1, z = 2
template<typename T>
inline TVector<T> TTransform<T>::GetUnitAxis(EAxis::Type InAxis) const
{
	if ( InAxis == EAxis::X )
	{
		return TransformVectorNoScale(TVector<T>(1.f, 0.f, 0.f));
	}
	else if ( InAxis == EAxis::Y )
	{
		return TransformVectorNoScale(TVector<T>(0.f, 1.f, 0.f));
	}

	return TransformVectorNoScale(TVector<T>(0.f, 0.f, 1.f));
}

template<typename T>
inline void TTransform<T>::Mirror(EAxis::Type MirrorAxis, EAxis::Type FlipAxis)
{
	// We do convert to Matrix for mirroring. 
	UE::Math::TMatrix<T> M = ToMatrixWithScale();
	M.Mirror(MirrorAxis, FlipAxis);
	SetFromMatrix(M);
}

/** same version of FMatrix::GetMaximumAxisScale function **/
/** @return the maximum magnitude of any row of the matrix. */
template<typename T>
inline float TTransform<T>::GetMaximumAxisScale() const
{
	DiagnosticCheckNaN_Scale3D();

	float Scale3DAbsMax;
	// Scale3DAbsXYZ1 = { Abs(X), Abs(Y)), Abs(Z), 0 }
	const TransformVectorRegister Scale3DAbsXYZ0 =  VectorAbs(Scale3D);
	// Scale3DAbsYZX1 = { Abs(Y),Abs(Z)),Abs(X), 0 }
	const TransformVectorRegister Scale3DAbsYZX0 = VectorSwizzle(Scale3DAbsXYZ0, 1,2,0,3);
	// Scale3DAbsZXY1 = { Abs(Z),Abs(X)),Abs(Y), 0 }
	const TransformVectorRegister Scale3DAbsZXY0 = VectorSwizzle(Scale3DAbsXYZ0, 2,0,1,3);
	// t0 = { Max(Abs(X), Abs(Y)),  Max(Abs(Y), Abs(Z)), Max(Abs(Z), Abs(X)), 0 }
	const TransformVectorRegister t0 = VectorMax(Scale3DAbsXYZ0, Scale3DAbsYZX0);
	// t1 = { Max(Abs(X), Abs(Y), Abs(Z)), Max(Abs(Y), Abs(Z), Abs(X)), Max(Abs(Z), Abs(X), Abs(Y)), 0 }
	const TransformVectorRegister t2 = VectorMax(t0, Scale3DAbsZXY0);
	// Scale3DAbsMax = Max(Abs(X), Abs(Y), Abs(Z));
	VectorStoreFloat1(t2, &Scale3DAbsMax);

	return Scale3DAbsMax;
}

/** @return the minimum magnitude of all components of the 3D scale. */
template<typename T>
inline float TTransform<T>::GetMinimumAxisScale() const
{
	DiagnosticCheckNaN_Scale3D();

	float Scale3DAbsMin;
	// Scale3DAbsXYZ1 = { Abs(X), Abs(Y)), Abs(Z), 0 }
	const TransformVectorRegister Scale3DAbsXYZ0 =  VectorAbs(Scale3D);
	// Scale3DAbsYZX1 = { Abs(Y),Abs(Z)),Abs(X), 0 }
	const TransformVectorRegister Scale3DAbsYZX0 = VectorSwizzle(Scale3DAbsXYZ0, 1,2,0,3);
	// Scale3DAbsZXY1 = { Abs(Z),Abs(X)),Abs(Y), 0 }
	const TransformVectorRegister Scale3DAbsZXY0 = VectorSwizzle(Scale3DAbsXYZ0, 2,0,1,3);
	// t0 = { Min(Abs(X), Abs(Y)),  Min(Abs(Y), Abs(Z)), Min(Abs(Z), Abs(X)), 0 }
	const TransformVectorRegister t0 = VectorMin(Scale3DAbsXYZ0, Scale3DAbsYZX0);
	// t1 = { Min(Abs(X), Abs(Y), Abs(Z)), Min(Abs(Y), Abs(Z), Abs(X)), Min(Abs(Z), Abs(X), Abs(Y)), 0 }
	const TransformVectorRegister t2 = VectorMin(t0, Scale3DAbsZXY0);
	// Scale3DAbsMax = Min(Abs(X), Abs(Y), Abs(Z));
	VectorStoreFloat1(t2, &Scale3DAbsMin);

	return Scale3DAbsMin;
}

/** 
 * mathematically if you have 0 scale, it should be infinite, 
 * however, in practice if you have 0 scale, and relative transform doesn't make much sense 
 * anymore because you should be instead of showing gigantic infinite mesh
 * also returning BIG_NUMBER causes sequential NaN issues by multiplying 
 * so we hardcode as 0
 */
template<typename T>
inline TVector<T> TTransform<T>::GetSafeScaleReciprocal(const TVector<T>& InScale, T Tolerance)
{
	TVector<T> SafeReciprocalScale;
	if (FMath::Abs(InScale.X) <= Tolerance)
	{
		SafeReciprocalScale.X = 0.f;
	}
	else
	{
		SafeReciprocalScale.X = 1.f/InScale.X;
	}

	if (FMath::Abs(InScale.Y) <= Tolerance)
	{
		SafeReciprocalScale.Y = 0.f;
	}
	else
	{
		SafeReciprocalScale.Y = 1.f/InScale.Y;
	}

	if (FMath::Abs(InScale.Z) <= Tolerance)
	{
		SafeReciprocalScale.Z = 0.f;
	}
	else
	{
		SafeReciprocalScale.Z = 1.f/InScale.Z;
	}

	return SafeReciprocalScale;
}


} // namespace UE::Math
} // namespace UE


#endif // ENABLE_VECTORIZED_TRANSFORM
