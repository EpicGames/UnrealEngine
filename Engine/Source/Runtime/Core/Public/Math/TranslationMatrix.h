// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector.h"
#include "Math/Plane.h"
#include "Math/Matrix.h"

namespace UE {
namespace Math {

template<typename T>
struct TTranslationMatrix
	: public TMatrix<T>
{
public:

	/** Constructor translation matrix based on given vector */
	[[nodiscard]] TTranslationMatrix(const TVector<T>& Delta);

	// Conversion to other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	[[nodiscard]] explicit TTranslationMatrix(const TTranslationMatrix<FArg>& From)
		: TMatrix<T>(From)
	{
	}
	
	/** Matrix factory. Return an FMatrix so we don't have type conversion issues in expressions. */
	[[nodiscard]] static TMatrix<T> Make(TVector<T> const& Delta)
	{
		return TTranslationMatrix<T>(Delta);
	}
};

template<typename T>
inline TTranslationMatrix<T>::TTranslationMatrix(const TVector<T>& Delta)
	: TMatrix<T>(
		TPlane<T>(1.0f,		0.0f,	0.0f,	0.0f),
		TPlane<T>(0.0f,		1.0f,	0.0f,	0.0f),
		TPlane<T>(0.0f,		0.0f,	1.0f,	0.0f),
		TPlane<T>(Delta.X,	Delta.Y,Delta.Z,1.0f)
	)
{ }
	
} // namespace Math
} // namespace UE

UE_DECLARE_LWC_TYPE(TranslationMatrix, 44);

template<> struct TIsUECoreVariant<FTranslationMatrix44f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FTranslationMatrix44d> { enum { Value = true }; };