// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Math/Vector.h"

/**
 * A pair of 3D vectors.
 */
struct FTwoVectors
{
public:

	/** Holds the first vector. */
	FVector	v1;

	/** Holds the second vector. */
	FVector	v2;

public:

	/** Default constructor. */
	[[nodiscard]] UE_FORCEINLINE_HINT	FTwoVectors();

	/**
	 * Creates and initializes a new instance with the specified vectors.
	 *
	 * @param In1 The first Vector.
	 * @param In2 The second Vector.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT	FTwoVectors( FVector In1, FVector In2 );

	/**
	 * Constructor.
	 *
	 * @param EForceInit Force Init Enum
	 */
	[[nodiscard]] explicit UE_FORCEINLINE_HINT FTwoVectors( EForceInit );

public:

	/**
	 * Gets result of addition of two pairs of vectors.
	 *
	 * @param V The pair to add.
	 * @return Result of addition.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT FTwoVectors operator+( const FTwoVectors& V ) const;

	/**
	 * Gets result of subtraction of two pairs of vectors.
	 *
	 * @param V The pair to subtract.
	 * @return Result of subtraction.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT FTwoVectors operator-( const FTwoVectors& V ) const;

	/**
	 * Gets result of scaling pair of vectors.
	 *
	 * @param Scale The scaling factor.
	 * @return Result of Scaling.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT FTwoVectors operator*( float Scale ) const;

	/**
	 * Gets result of dividing pair of vectors.
	 *
	 * @param Scale What to divide by.
	 * @return Result of division.
	 */
	[[nodiscard]] FTwoVectors operator/( float Scale ) const;

	/**
	 * Gets result of multiplying two pairs of vectors.
	 *
	 * @param V The pair to multiply with.
	 * @return Result of multiplication.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT FTwoVectors operator*( const FTwoVectors& V ) const;

	/**
	 * Gets result of division of two pairs of vectors.
	 *
	 * @param V The pair to divide by.
	 * @return Result of division.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT FTwoVectors operator/( const FTwoVectors& V ) const;

	// Binary comparison operators.

	/**
	 * Checks two pairs for equality.
	 *
	 * @param V The other pair.
	 * @return true if the two pairs are equal, false otherwise..
	 */
	[[nodiscard]] bool operator==( const FTwoVectors& V ) const;

	/**
	 * Checks two pairs for inequality.
	 *
	 * @param V The other pair.
	 * @return true if the two pairs are different, false otherwise..
	 */
	[[nodiscard]] bool operator!=( const FTwoVectors& V ) const;

	/**
	 * Error-tolerant comparison.
	 *
	 * @param V The other pair.
	 * @param Tolerance Error Tolerance.
	 * @return true if two pairs are equal within specified tolerance, false otherwise..
	 */
	[[nodiscard]] bool Equals( const FTwoVectors& V, float Tolerance = UE_KINDA_SMALL_NUMBER ) const;

	// Unary operators.

	/**
	 * Get a negated copy of the pair.
	 *
	 * @return A negated copy of the pair.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT FTwoVectors operator-() const;

	// Assignment operators.

	/**
	 * Add a pair to this.
	 *
	 * @param The pair to add.
	 * @return Copy of the pair after addition.
	 */
	UE_FORCEINLINE_HINT FTwoVectors operator+=( const FTwoVectors& V );

	/**
	 * Subtract a pair from this.
	 *
	 * @param The pair to subtract.
	 * @return Copy of the pair after subtraction.
	 */
	UE_FORCEINLINE_HINT FTwoVectors operator-=( const FTwoVectors& V );

	/**
	 * Scale the pair.
	 *
	 * @param Scale What to scale by.
	 * @return Copy of the pair after scaling.
	 */
	UE_FORCEINLINE_HINT FTwoVectors operator*=( float Scale );

	/**
	 * Divide the pair.
	 *
	 * @param What to divide by.
	 * @return Copy of the pair after division.
	 */
	FTwoVectors operator/=( float V );

	/**
	 * Multiply the pair by another.
	 *
	 * @param The other pair.
	 * @return Copy of the pair after multiplication.
	 */
	FTwoVectors operator*=( const FTwoVectors& V );

	/**
	 * Divide the pair by another.
	 *
	 * @param The other pair.
	 * @return Copy of the pair after multiplication.
	 */
	FTwoVectors operator/=( const FTwoVectors& V );

	/**
	 * Get a specific component from the pair.
	 *
	 * @param i The index of the component, even indices are for the first vector,
	 *			odd ones are for the second. Returns index 5 if out of range.
	 * @return Reference to the specified component.
	 */
    FVector::FReal& operator[]( int32 i );

public:

	/**
	 * Get the maximum value of all the vector coordinates.
	 *
	 * @return The maximum value of all the vector coordinates.
	 */
	[[nodiscard]] FVector::FReal GetMax() const;

	/**
	 * Get the minimum value of all the vector coordinates.
	 *
	 * @return The minimum value of all the vector coordinates.
	 */
	[[nodiscard]] FVector::FReal GetMin() const;

	/**
	 * Get a textual representation of this two-vector.
	 *
	 * @return A string describing the two-vector.
	 */
	[[nodiscard]] FString ToString() const;

public:

	/**
	 * Serializes the two-vector.
	 *
	 * @param Ar The archive to serialize into.
	 * @param TwoVectors The two-vector to serialize.
	 * @return Reference to the Archive after serialization.
	 */
	friend FArchive& operator<<( FArchive& Ar, FTwoVectors& TwoVectors )
	{
		return Ar << TwoVectors.v1 << TwoVectors.v2;
	}

	bool Serialize( FArchive& Ar )
	{
		Ar << *this;
		return true;
	}
};


/* FTwoVectors inline functions
 *****************************************************************************/

UE_FORCEINLINE_HINT FTwoVectors operator*( float Scale, const FTwoVectors& V )
{
	return V.operator*( Scale );
}


inline	FTwoVectors::FTwoVectors() :
	v1(0.0f),
	v2(0.0f)
{ }


inline FTwoVectors::FTwoVectors( FVector In1, FVector In2 )
	: v1(In1)
	, v2(In2)
{ }


inline FTwoVectors::FTwoVectors( EForceInit )
	: v1(ForceInit)
	, v2(ForceInit)
{ }


inline FTwoVectors FTwoVectors::operator+( const FTwoVectors& V ) const
{
	return FTwoVectors(
		FVector(v1 + V.v1),
		FVector(v2 + V.v2)
	);
}


inline FTwoVectors FTwoVectors::operator-( const FTwoVectors& V ) const
{
	return FTwoVectors( 
		FVector(v1 - V.v1),
		FVector(v2 - V.v2)
	);
}


inline FTwoVectors FTwoVectors::operator*( float Scale ) const
{
	return FTwoVectors(
		FVector(v1 * Scale),
		FVector(v2 * Scale)
	);
}


inline FTwoVectors FTwoVectors::operator/( float Scale ) const
{
	const float RScale = 1.f / Scale;

	return FTwoVectors(
		FVector(v1 * RScale), 
		FVector(v2 * RScale)
	);
}


inline FTwoVectors FTwoVectors::operator*( const FTwoVectors& V ) const
{
	return FTwoVectors(
		FVector(v1 * V.v1),
		FVector(v2 * V.v2)
		);
}


inline FTwoVectors FTwoVectors::operator/( const FTwoVectors& V ) const
{
	return FTwoVectors(
		FVector(v1 / V.v1),
		FVector(v2 / V.v2)
	);
}


UE_FORCEINLINE_HINT bool FTwoVectors::operator==( const FTwoVectors& V ) const
{
	return ((v1 == V.v1) && (v2 == V.v2));
}


UE_FORCEINLINE_HINT bool FTwoVectors::operator!=( const FTwoVectors& V ) const
{
	return ((v1 != V.v1) || (v2 != V.v2));
}


UE_FORCEINLINE_HINT bool FTwoVectors::Equals( const FTwoVectors& V, float Tolerance ) const
{
	return v1.Equals(V.v1, Tolerance) && v2.Equals(V.v2, Tolerance);
}


inline FTwoVectors FTwoVectors::operator-() const
{
	return FTwoVectors(
		FVector(-v1),
		FVector(-v2)
	);
}


inline FTwoVectors FTwoVectors::operator+=( const FTwoVectors& V )
{
	v1 += V.v1;
	v2 += V.v2;

	return *this;
}


inline FTwoVectors FTwoVectors::operator-=( const FTwoVectors& V )
{
	v1 -= V.v1;
	v2 -= V.v2;

	return *this;
}


inline FTwoVectors FTwoVectors::operator*=( float Scale )
{
	v1 *= Scale;
	v2 *= Scale;

	return *this;
}


inline FTwoVectors FTwoVectors::operator/=( float V )
{
	const float RV = 1.f/V;

	v1 *= RV;
	v2 *= RV;

	return *this;
}


inline FTwoVectors FTwoVectors::operator*=( const FTwoVectors& V )
{
	v1 *= V.v1;
	v2 *= V.v2;
	return *this;
}


inline FTwoVectors FTwoVectors::operator/=( const FTwoVectors& V )
{
	v1 /= V.v1;
	v2 /= V.v2;

	return *this;
}


inline FVector::FReal FTwoVectors::GetMax() const
{
	const FVector::FReal MaxMax = FMath::Max(FMath::Max(v1.X, v1.Y), v1.Z);
	const FVector::FReal MaxMin = FMath::Max(FMath::Max(v2.X, v2.Y), v2.Z);

	return FMath::Max(MaxMax, MaxMin);
}


inline FVector::FReal FTwoVectors::GetMin() const
{
	const FVector::FReal MinMax = FMath::Min(FMath::Min(v1.X, v1.Y), v1.Z);
	const FVector::FReal MinMin = FMath::Min(FMath::Min(v2.X, v2.Y), v2.Z);

	return FMath::Min(MinMax, MinMin);
}


inline FVector::FReal& FTwoVectors::operator[]( int32 i )
{
	check(i > -1);
	check(i < 6);

	switch(i)
	{
	case 0:		return v1.X;
	case 1:		return v2.X;
	case 2:		return v1.Y;
	case 3:		return v2.Y;
	case 4:		return v1.Z;
	default:	return v2.Z;
	}
}


UE_FORCEINLINE_HINT FString FTwoVectors::ToString() const
{
	return FString::Printf(TEXT("V1=(%s) V2=(%s)"), *v1.ToString(), *v2.ToString());
}

template <> struct TIsPODType<FTwoVectors> { enum { Value = true }; };
