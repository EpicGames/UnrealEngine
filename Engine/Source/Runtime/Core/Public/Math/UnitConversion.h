// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Templates/ValueOrError.h"
#include "UObject/CoreReflectedTypeAccessors.h"

class FText;
template<typename NumericType> struct FNumericUnit;

/** Enum *must* be zero-indexed and sequential. Must be grouped by relevance and ordered by magnitude. */
/** Enum *must* match the mirrored enum that exists in CoreUObject/NoExportTypes.h for the purposes of UObject reflection */
enum class EUnit : uint8
{
	/** Scalar distance/length unit. */
	Micrometers, Millimeters, Centimeters, Meters, Kilometers,
	Inches, Feet, Yards, Miles,
	Lightyears,

	/** Angular unit. */
	Degrees, Radians,

	/** Speed unit. */
	CentimetersPerSecond, MetersPerSecond, KilometersPerHour, MilesPerHour,

	/** Angular speed unit. */
	DegreesPerSecond, RadiansPerSecond,

	/** Acceleration unit. */
	CentimetersPerSecondSquared, MetersPerSecondSquared,

	/** Temperature unit. */
	Celsius, Farenheit, Kelvin,

	/** Mass unit. */
	Micrograms, Milligrams, Grams, Kilograms, MetricTons,
	Ounces, Pounds, Stones,

	/** Density unit. */
	GramsPerCubicCentimeter, GramsPerCubicMeter, KilogramsPerCubicCentimeter, KilogramsPerCubicMeter,

	/** Force unit. */
	Newtons, PoundsForce, KilogramsForce, KilogramCentimetersPerSecondSquared,

	/** Torque unit. */
	NewtonMeters, KilogramCentimetersSquaredPerSecondSquared,

	/** Impulse unit. */
	NewtonSeconds, KilogramCentimeters, KilogramMeters,

	/** Loudness unit. */
	Decibels,
	
	/** Frequency unit. */
	Hertz, Kilohertz, Megahertz, Gigahertz, RevolutionsPerMinute,
	
	/** Data Size unit. */
	Bytes, Kilobytes, Megabytes, Gigabytes, Terabytes,

	/** Luminous flux unit. */
	Lumens,
	
	/** Luminous intensity unit. */
	Candela,
	
	/** Illuminance unit. */
	Lux,
	
	/** Luminance unit. */
	CandelaPerMeter2, Nit,
	
	/** Exposure value unit. */
	ExposureValue,

	/** Time unit. */
	Nanoseconds, Microseconds, Milliseconds, Seconds, Minutes, Hours, Days, Months, Years,

	/** Pixel density unit. */
	PixelsPerInch,

	/** Percentage. */
	Percentage,

	/** Arbitrary multiplier. */
	Multiplier,

	/** Stress unit. */
	Pascals, KiloPascals, MegaPascals, GigaPascals,

	/** Symbolic entry, not specifiable on meta data. */
	Unspecified
};

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<EUnit>();
#endif

/** Enumeration that specifies particular classes of unit */
enum class EUnitType
{
	Distance, Angle, Speed, AngularSpeed, Acceleration, Temperature, Mass, Density, Force, Torque, Impulse, PositionalImpulse, Loudness, Frequency, DataSize, LuminousFlux, LuminousIntensity, Illuminance, Luminance, Time, PixelDensity, Multipliers, ExposureValue, Stress,

	// Symbolic entry - do not use directly
	NumberOf,
};

template<typename NumericType> struct FNumericUnit;

/** Unit settings accessed globally through FUnitConversion::Settings() */
class FUnitSettings
{
public:

	[[nodiscard]] CORE_API FUnitSettings();

	/** Check whether unit display is globally enabled or disabled */
	CORE_API bool ShouldDisplayUnits() const;
	CORE_API void SetShouldDisplayUnits(bool bInGlobalUnitDisplay);
	
	/** Get/Set the specific valid units to display the specified type of unit in */
	CORE_API const TArray<EUnit>& GetDisplayUnits(EUnitType InType) const;
	CORE_API void SetDisplayUnits(EUnitType InType, const TArray<EUnit>& Units);
	CORE_API void SetDisplayUnits(EUnitType InType, EUnit Units);

	/** Returns an event delegate that is executed when a display setting has changed. (GlobalUnitDisplay or DefaultInputUnits) */
	DECLARE_EVENT(FUnitSettings, FDisplaySettingChanged);
	FDisplaySettingChanged& OnDisplaySettingsChanged() { return SettingChangedEvent; }

private:

	/** Global toggle controlling whether we should display units or not */
	bool bGlobalUnitDisplay;

	/** Arrays of units that are valid to display on interfaces */
	TArray<EUnit> DisplayUnits[(uint8)EUnitType::NumberOf + 1];
	
	/** Holds an event delegate that is executed when a display setting has changed. */
	FDisplaySettingChanged SettingChangedEvent;
};

struct FUnitConversion
{
	/** Get the global settings for unit conversion/display */
	[[nodiscard]] static CORE_API FUnitSettings& Settings();

	/** Check whether it is possible to convert a number between the two specified units */
	[[nodiscard]] static CORE_API bool AreUnitsCompatible(EUnit From, EUnit To);

	/** Check whether a unit is of the specified type */
	[[nodiscard]] static CORE_API bool IsUnitOfType(EUnit Unit, EUnitType Type);

	/** Get the type of the specified unit */
	[[nodiscard]] static CORE_API EUnitType GetUnitType(EUnit);

	/** Get the display string for the the specified unit type */
	[[nodiscard]] static CORE_API const TCHAR* GetUnitDisplayString(EUnit Unit);

	/** Helper function to find a unit from a string (name or display string) */
	[[nodiscard]] static CORE_API TOptional<EUnit> UnitFromString(const TCHAR* UnitString);

	/** Helper function to get all supported units */
	[[nodiscard]] static CORE_API TConstArrayView<const TCHAR*> GetSupportedUnits();

public:

	/** Convert the specified number from one unit to another. Does nothing if the units are incompatible. */
	template<typename T>
	[[nodiscard]] static T Convert(T InValue, EUnit From, EUnit To);

	/** Quantizes this number to the most appropriate unit for user friendly presentation (e.g. 1000m returns 1km). */
	template<typename T>
	[[nodiscard]] static FNumericUnit<T> QuantizeUnitsToBestFit(T Value, EUnit Units);

	/** Quantizes this number to the most appropriate unit for user friendly presentation (e.g. 1000m returns 1km), adhereing to global display settings. */
	template<typename T>
	[[nodiscard]] static EUnit CalculateDisplayUnit(T Value, EUnit InUnits);

};


/**
 * FNumericUnit is a numeric type that wraps the templated type, whilst a specified unit.
 * It handles conversion to/from related units automatically. The units are considered not to contribute to the type's state, and as such should be considered immutable once set.
 */
template<typename NumericType>
struct FNumericUnit
{
	/** The numeric (scalar) value */
	NumericType Value;
	/** The associated units for the value. Can never change once set to anything other than EUnit::Unspecified. */
	const EUnit Units;

	/** Constructors */
	[[nodiscard]] FNumericUnit();
	[[nodiscard]] FNumericUnit(const NumericType& InValue, EUnit InUnits = EUnit::Unspecified);

	/** Copy construction/assignment from the same type */
	[[nodiscard]] FNumericUnit(const FNumericUnit& Other);
	FNumericUnit& operator=(const FNumericUnit& Other);

	/** Templated Copy construction/assignment from differing numeric types. Relies on implicit conversion of the two numeric types. */
	template<typename OtherType>
	[[nodiscard]] FNumericUnit(const FNumericUnit<OtherType>& Other);
	template<typename OtherType>
	FNumericUnit& operator=(const FNumericUnit<OtherType>& Other);

	/** Convert this quantity to a different unit */
	[[nodiscard]] TOptional<FNumericUnit<NumericType>> ConvertTo(EUnit ToUnits) const;

public:

	/** Quantizes this number to the most appropriate unit for user friendly presentation (e.g. 1000m returns 1km). */
	[[nodiscard]] FNumericUnit<NumericType> QuantizeUnitsToBestFit() const;

	/** Try and parse an expression into a numeric unit */
	static TValueOrError<FNumericUnit<NumericType>, FText> TryParseExpression(const TCHAR* InExpression, EUnit InDefaultUnit, const FNumericUnit<NumericType>& InExistingValue);

	/** Parse a numeric unit from a string */
	static TOptional<FNumericUnit<NumericType>> TryParseString(const TCHAR* InSource);

private:
	/** Conversion to the numeric type disabled as coupled with implicit construction from NumericType can easily lead to loss of associated units. */
	operator const NumericType&() const = delete;

	/** Copy another unit into this one, taking account of its units, and applying necessary conversion */
	template<typename OtherType>
	void CopyValueWithConversion(const FNumericUnit<OtherType>& Other);

	/** Given a string, skip past whitespace, then any numeric characters. Set End pointer to the end of the last numeric character. */
	static bool ExtractNumberBoundary(const TCHAR* Start, const TCHAR*& End);

	/** Global arithmetic operators for number types. Deals with conversion from related units correctly. Note must be inlined for hidden friend optimization to work */
	template<typename OtherType>
	[[nodiscard]] friend inline bool operator==(const FNumericUnit<NumericType>& LHS, const FNumericUnit<OtherType>& RHS)
	{
		if (LHS.Units != EUnit::Unspecified && RHS.Units != EUnit::Unspecified)
		{
			if (LHS.Units == RHS.Units)
			{
				return LHS.Value == RHS.Value;
			}
			else if (FUnitConversion::AreUnitsCompatible(LHS.Units, RHS.Units))
			{
				return LHS.Value == FUnitConversion::Convert(RHS.Value, RHS.Units, LHS.Units);
			}
			else
			{
				// Invalid conversion
				return false;
			}
		}
		else
		{
			return LHS.Value == RHS.Value;
		}
	}

	template<typename OtherType>
	[[nodiscard]] friend inline bool operator!=(const FNumericUnit<NumericType>& LHS, const FNumericUnit<OtherType>& RHS)
	{
		return !(LHS == RHS);
	}
};

template <typename CharType, typename T>
TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const FNumericUnit<T>& NumericUnit);

template<typename T>
FString LexToString(const FNumericUnit<T>& NumericUnit);

template<typename T>
FString LexToSanitizedString(const FNumericUnit<T>& NumericUnit);

template<typename T>
void LexFromString(FNumericUnit<T>& OutValue, const TCHAR* String);
	
template<typename T>
bool LexTryParseString(FNumericUnit<T>& OutValue, const TCHAR* String);

// Include template definitions
#include "Math/UnitConversion.inl" // IWYU pragma: export
