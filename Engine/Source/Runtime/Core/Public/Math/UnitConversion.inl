// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** Inline file for UnitConversion.h to separate the implementation from the header */


#include "CoreTypes.h"
#include "CoreFwd.h"
#include "Misc/OptionalFwd.h"

struct FMath;
struct FUnitConversion;
enum class EUnit : uint8;
enum class EUnitType;
template<typename NumericType> struct FNumericUnit;
template<typename ValueType, typename ErrorType> class TValueOrError;

namespace UnitConversion
{
	/** Find the common quantization factor for the specified distance unit. Quantizes to Meters. */
	CORE_API double DistanceUnificationFactor(EUnit From);
	/** Find the common quantization factor for the specified angular unit. Quantizes to Degrees. */
	CORE_API double AngleUnificationFactor(EUnit From);
	/** Find the common quantization factor for the specified speed unit. Quantizes to km/h. */
	CORE_API double SpeedUnificationFactor(EUnit From);
	/** Find the common quantization factor for the specified angular speed unit. Quantizes to DegreesPerSecond. */
	CORE_API double AngularSpeedUnificationFactor(EUnit From);
	/** Find the common quantization factor for the specified acceleration unit. Quantizes to m/s2. */
	CORE_API double AccelerationUnificationFactor(EUnit From);
	/** Find the common quantization factor for the specified mass unit. Quantizes to Grams. */
	CORE_API double MassUnificationFactor(EUnit From);
	/** Find the common quantization factor for the specified density unit. Quantizes to GramsPerCubicCentimeter. */
	CORE_API double DensityUnificationFactor(EUnit From);
	/** Find the common quantization factor for the specified force unit. Quantizes to Newtons. */
	CORE_API double ForceUnificationFactor(EUnit From);
	/** Find the common quantization factor for the specified torque unit. Quantizes to Newton meters. */
	CORE_API double TorqueUnificationFactor(EUnit From);
	/** Find the common quantization factor for the specified position impulse unit. Quantizes to kilogram centimeters. */
	CORE_API double PositionalImpulseUnificationFactor(EUnit From);
	/** Find the common quantization factor for the specified frequency unit. Quantizes to KHz. */
	CORE_API double FrequencyUnificationFactor(EUnit From);
	/** Find the common quantization factor for the specified data size unit. Quantizes to MB. */
	CORE_API double DataSizeUnificationFactor(EUnit From);
	/** Find the common quantization factor for the specified time unit. Quantizes to hours. */
	CORE_API double TimeUnificationFactor(EUnit From);
	/** Find the common quantization factor for the specified multiplier unit. Quantizes to 1.0 scale (where 1.0 == 100%). */
	CORE_API double MultiplierUnificationFactor(EUnit From);
	/** Find the common quantization factor for the specified multiplier unit. Quantizes to 1.0 scale (where 1.0 == 100%). */
	CORE_API double StressUnificationFactor(EUnit From);

	/** Attempt to parse an expression into a numeric unit */
	CORE_API TValueOrError<FNumericUnit<double>, FText> TryParseExpression(const TCHAR* InExpression, EUnit DefaultUnit, const FNumericUnit<double>& InExistingValue);

	/** Structure used to define the factor required to get from one unit type to the next. */
	struct FQuantizationInfo
	{
		/** The unit to which this factor applies */
		EUnit Units;
		/** The factor by which to multiply to get to the next unit in this range */
		float Factor;

		/** Constructor */
		FQuantizationInfo(EUnit InUnit, float InFactor) : Units(InUnit), Factor(InFactor) {}
	};

	/** Find the quantization bounds for the specified unit, if any */
	CORE_API TOptional<const TArray<FQuantizationInfo>*> GetQuantizationBounds(EUnit Unit);

}	// namespace UnitConversion

/** Convert the specified number from one unit to another. Does nothing if the units are incompatible. */
template<typename T>
T FUnitConversion::Convert(T InValue, EUnit From, EUnit To)
{
	using namespace UnitConversion;

	if (!AreUnitsCompatible(From, To))
	{
		return InValue;
	}
	else if (From == EUnit::Unspecified || To == EUnit::Unspecified)
	{
		return InValue;
	}
	else if (From == To)
	{
		return InValue;
	}

	switch(FUnitConversion::GetUnitType(From))
	{
		case EUnitType::Distance:			return static_cast<T>(InValue * DistanceUnificationFactor(From)				* (1.0 / DistanceUnificationFactor(To)));
		case EUnitType::Angle:				return static_cast<T>(InValue * AngleUnificationFactor(From) 				* (1.0 / AngleUnificationFactor(To)));
		case EUnitType::Speed:				return static_cast<T>(InValue * SpeedUnificationFactor(From) 				* (1.0 / SpeedUnificationFactor(To)));
		case EUnitType::AngularSpeed:		return static_cast<T>(InValue * AngularSpeedUnificationFactor(From)			* (1.0 / AngularSpeedUnificationFactor(To)));
		case EUnitType::Acceleration:		return static_cast<T>(InValue * AccelerationUnificationFactor(From)			* (1.0 / AccelerationUnificationFactor(To)));
		case EUnitType::Mass:				return static_cast<T>(InValue * MassUnificationFactor(From) 				* (1.0 / MassUnificationFactor(To)));
		case EUnitType::Density:			return static_cast<T>(InValue * DensityUnificationFactor(From)				* (1.0 / DensityUnificationFactor(To)));
		case EUnitType::Force:				return static_cast<T>(InValue * ForceUnificationFactor(From) 				* (1.0 / ForceUnificationFactor(To)));
		case EUnitType::Torque:				return static_cast<T>(InValue * TorqueUnificationFactor(From)				* (1.0 / TorqueUnificationFactor(To)));
		case EUnitType::PositionalImpulse:	return static_cast<T>(InValue * PositionalImpulseUnificationFactor(From)	* (1.0 / PositionalImpulseUnificationFactor(To)));
		case EUnitType::Frequency:			return static_cast<T>(InValue * FrequencyUnificationFactor(From) 			* (1.0 / FrequencyUnificationFactor(To)));
		case EUnitType::DataSize:			return static_cast<T>(InValue * DataSizeUnificationFactor(From) 			* (1.0 / DataSizeUnificationFactor(To)));
		case EUnitType::LuminousFlux:		return InValue;	
		case EUnitType::Time:				return static_cast<T>(InValue * TimeUnificationFactor(From) 				* (1.0 / TimeUnificationFactor(To)));
		case EUnitType::Multipliers:		return static_cast<T>(InValue * MultiplierUnificationFactor(From) 			* (1.0 / MultiplierUnificationFactor(To)));
		case EUnitType::Stress:				return static_cast<T>(InValue * StressUnificationFactor(From)				* (1.0 / StressUnificationFactor(To)));
		// Temperature conversion is not just a simple multiplication, so needs special treatment
		case EUnitType::Temperature:
		{
			double NewValue = InValue;
			// Put it into kelvin
			if (From == EUnit::Celsius)
			{
				NewValue = NewValue + 273.15;
			}
			else if (From == EUnit::Farenheit)
			{
				NewValue = (NewValue + 459.67) * 5.0/9.0;
			}
			// And out again
			if (To == EUnit::Celsius)
			{
				return static_cast<T>(NewValue - 273.15);
			}
			else if (To == EUnit::Farenheit)
			{
				return static_cast<T>(NewValue * 9.0/5.0 - 459.67);
			}
			else
			{
				return static_cast<T>(NewValue);
			}
		}

		case EUnitType::Loudness:
		case EUnitType::Impulse:
		case EUnitType::LuminousIntensity:
		case EUnitType::Illuminance:
		case EUnitType::Luminance:
		case EUnitType::PixelDensity:
		case EUnitType::ExposureValue:
		case EUnitType::NumberOf:
		default:							return InValue;
	}
}

template<typename T>
FNumericUnit<T> FUnitConversion::QuantizeUnitsToBestFit(T Value, EUnit Units)
{
	auto OptionalBounds = UnitConversion::GetQuantizationBounds(Units);
	if (!OptionalBounds.IsSet())
	{
		return FNumericUnit<T>(Value, Units);
	}

	const auto& Bounds = *OptionalBounds.GetValue();

	const int32 CurrentUnitIndex = (uint8)Units - (uint8)Bounds[0].Units;

	EUnit NewUnits = Units;
	double NewValue = Value;

	if (FMath::Abs(NewValue) > 1)
	{
		// Large number? Try larger units
		for (int32 Index = CurrentUnitIndex; Index < Bounds.Num(); ++Index)
		{
			if (Bounds[Index].Factor == 0)
			{
				break;
			}

			const auto Tmp = NewValue / Bounds[Index].Factor;

			if (FMath::Abs(Tmp) < 1)
			{
				break;
			}

			NewValue = Tmp;
			NewUnits = Bounds[Index + 1].Units;
		}
	}
	else if (NewValue != 0)
	{
		// Small number? Try smaller units
		for (int32 Index = CurrentUnitIndex - 1; Index >= 0; --Index)
		{
			NewValue *= Bounds[Index].Factor;
			NewUnits = Bounds[Index].Units;

			if (FMath::Abs(NewValue) > 1)
			{
				break;
			}
		}
	}

	return FNumericUnit<T>(NewValue, NewUnits);
}

template<typename T>
EUnit FUnitConversion::CalculateDisplayUnit(T Value, EUnit InUnits)
{
	if (InUnits == EUnit::Unspecified)
	{
		return EUnit::Unspecified;
	}

	const TArray<EUnit>& DisplayUnits = Settings().GetDisplayUnits(GetUnitType(InUnits));
	if (DisplayUnits.Num() == 0)
	{
		return QuantizeUnitsToBestFit(Value, InUnits).Units;
	}
	if (DisplayUnits.Num() == 1)
	{
		return DisplayUnits[0];
	}

	// If the value we were given was 0, change it to something we can actually work with
	if (Value == 0)
	{
		Value = 1;
	}

	int32 BestIndex = 0;
	for (int32 Index = 0; Index < DisplayUnits.Num() - 1; ++Index)
	{
		T Best, Next;
		if constexpr (std::is_signed_v<T>)
		{
			Best = FMath::Abs(Convert(Value, InUnits, DisplayUnits[BestIndex]));
			Next = FMath::Abs(Convert(Value, InUnits, DisplayUnits[Index + 1]));
		}
		else
		{
			Best = Convert(Value, InUnits, DisplayUnits[BestIndex]);
			Next = Convert(Value, InUnits, DisplayUnits[Index + 1]);
		}

		if (Best < 1.0 && Next >= 1.0)
		{
			BestIndex = Index + 1;
		}
		else if (Best < 1.0 && Next < 1.0)
		{
			if (Next > Best)
			{
				BestIndex = Index + 1;
			}
		}
		else if (Best >= 1.0 && Next >= 1.0)
		{
			if (FMath::LogX(10.0f, Next) < FMath::LogX(10.0f, Best))
			{
				BestIndex = Index + 1;
			}
		}
	}

	return DisplayUnits[BestIndex];
}

template<typename NumericType>
FNumericUnit<NumericType>::FNumericUnit()
	: Units(EUnit::Unspecified)
{}

template<typename NumericType>
FNumericUnit<NumericType>::FNumericUnit(const NumericType& InValue, EUnit InUnits)
	: Value(InValue), Units(InUnits)
{}

template<typename NumericType>
FNumericUnit<NumericType>::FNumericUnit(const FNumericUnit& Other)
	: Units(EUnit::Unspecified)
{
	(*this) = Other;
}

template<typename NumericType>
FNumericUnit<NumericType>& FNumericUnit<NumericType>::operator=(const FNumericUnit& Other)
{
	CopyValueWithConversion(Other);
	return *this;
}

/** Templated Copy construction/assignment from differing numeric types. Relies on implicit conversion of the two numeric types. */
template<typename NumericType> template<typename OtherType>
FNumericUnit<NumericType>::FNumericUnit(const FNumericUnit<OtherType>& Other)
	: Units(EUnit::Unspecified)
{
	(*this) = Other;
}

template<typename NumericType> template<typename OtherType>
FNumericUnit<NumericType>& FNumericUnit<NumericType>::operator=(const FNumericUnit<OtherType>& Other)
{
	CopyValueWithConversion(Other);
	return *this;
}

/** Convert this quantity to a different unit */
template<typename NumericType>
TOptional<FNumericUnit<NumericType>> FNumericUnit<NumericType>::ConvertTo(EUnit ToUnits) const
{
	if (Units == EUnit::Unspecified)
	{
		return FNumericUnit(Value, ToUnits);
	}
	else if (FUnitConversion::AreUnitsCompatible(Units, ToUnits))
	{
		return FNumericUnit<NumericType>(FUnitConversion::Convert(Value, Units, ToUnits), ToUnits);
	}

	return TOptional<FNumericUnit<NumericType>>();
}

template<typename NumericType>
FNumericUnit<NumericType> FNumericUnit<NumericType>::QuantizeUnitsToBestFit() const
{
	return FUnitConversion::QuantizeUnitsToBestFit(Value, Units);
}

template<typename NumericType>
TValueOrError<FNumericUnit<NumericType>, FText> FNumericUnit<NumericType>::TryParseExpression(const TCHAR* InExpression, EUnit InDefaultUnit, const FNumericUnit<NumericType>& InExistingValue)
{
	TValueOrError<FNumericUnit<double>, FText> Result = UnitConversion::TryParseExpression(InExpression, InDefaultUnit, InExistingValue);
	if (Result.IsValid())
	{
		const auto& Value = Result.GetValue();
		return MakeValue(FNumericUnit<NumericType>((NumericType)Value.Value, Value.Units));
	}

	return MakeError(Result.GetError());
}

template<typename NumericType>
TOptional<FNumericUnit<NumericType>> FNumericUnit<NumericType>::TryParseString(const TCHAR* InSource)
{
	TOptional<FNumericUnit<NumericType>> Result;
	if (!InSource || !*InSource)
	{
		return Result;
	}

	const TCHAR* NumberEnd = nullptr;
	if (!ExtractNumberBoundary(InSource, NumberEnd))
	{
		return Result;
	}

	NumericType NewValue;
	LexFromString(NewValue, InSource);

	// Now parse the units
	while(FChar::IsWhitespace(*NumberEnd)) ++NumberEnd;

	if (*NumberEnd == '\0')
	{
		// No units
		Result.Emplace(NewValue);
	}
	else
	{
		// If the string specifies units, they must map to something that exists for this function to succeed
		auto NewUnits = FUnitConversion::UnitFromString(NumberEnd);
		if (NewUnits)
		{
			Result.Emplace(NewValue, NewUnits.GetValue());
		}
	}

	return Result;
}

/** Copy another unit into this one, taking account of its units, and applying necessary conversion */
template<typename NumericType> template<typename OtherType>
void FNumericUnit<NumericType>::CopyValueWithConversion(const FNumericUnit<OtherType>& Other)
{
	if (Units != EUnit::Unspecified && Other.Units != EUnit::Unspecified)
	{
		if (Units == Other.Units)
		{
			Value = Other.Value;
		}
		else if (FUnitConversion::AreUnitsCompatible(Units, Other.Units))
		{
			Value = FUnitConversion::Convert(Other.Value, Other.Units, Units);
		}
		else
		{
			// Invalid conversion - assignment invalid
		}
	}
	else
	{
		// If our units haven't been specified, we take on the units of the rhs
		if (Units == EUnit::Unspecified)
		{
			// This is the only time we ever change units. Const_cast is 'acceptible' here since the units haven't been specified yet.
			const_cast<EUnit&>(Units) = Other.Units;
		}

		Value = Other.Value;
	}
}

template<typename NumericType>
bool FNumericUnit<NumericType>::ExtractNumberBoundary(const TCHAR* Start, const TCHAR*& End)
{
	while(FChar::IsWhitespace(*Start)) ++Start;

	End = Start;
	if (*End == '-' || *End == '+')
	{
		End++;
	}

	bool bHasDot = false;
	while (FChar::IsDigit(*End) || *End == '.')
	{
		if (*End == '.')
		{
			if (bHasDot)
			{
				return false;
			}
			bHasDot = true;
		}
		++End;
	}

	return true;
}

template <typename NumericType>
struct TNumericLimits<FNumericUnit<NumericType>> : public TNumericLimits<NumericType>
{ };

template <typename CharType, typename T>
TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const FNumericUnit<T>& NumericUnit)
{
	Builder << NumericUnit.Value;
	Builder << ANSITEXTVIEW(" ");
	Builder << FUnitConversion::GetUnitDisplayString(NumericUnit.Units);
	return Builder;
}

template<typename T>
FString LexToString(const FNumericUnit<T>& NumericUnit)
{
	TStringBuilder<128> Builder;
	Builder << NumericUnit;
	return FString(Builder);
}

template<typename T>
FString LexToSanitizedString(const FNumericUnit<T>& NumericUnit)
{
	FString String = LexToSanitizedString(NumericUnit.Value);
	String += TEXT(" ");
	String += FUnitConversion::GetUnitDisplayString(NumericUnit.Units);

	return String;
}

template<typename T>
void LexFromString(FNumericUnit<T>& OutValue, const TCHAR* String)
{
	auto Parsed = FNumericUnit<T>::TryParseString(String);
	if (Parsed)
	{
		OutValue = Parsed.GetValue();
	}
}
	
template<typename T>
bool LexTryParseString(FNumericUnit<T>& OutValue, const TCHAR* String)
{
	auto Parsed = FNumericUnit<T>::TryParseString(String);
	if (Parsed)
	{
		OutValue = Parsed.GetValue();
		return true;
	}

	return false;
}
