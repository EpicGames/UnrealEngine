// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_exports
#include "Math/Axis.h"
#include "Math/Box.h"
#include "Math/Box2D.h"
#include "Math/BoxSphereBounds.h"
#include "Math/CapsuleShape.h"
#include "Math/ClipProjectionMatrix.h"
#include "Math/Color.h"
#include "Math/ColorList.h"
#include "Math/ConvexHull2d.h"
#include "Math/CurveEdInterface.h"
#include "Math/Edge.h"
#include "Math/Float16.h"
#include "Math/Float16Color.h"
#include "Math/Float32.h"
#include "Math/IntPoint.h"
#include "Math/IntRect.h"
#include "Math/IntVector.h"
#include "Math/InterpCurve.h"
#include "Math/InterpCurvePoint.h"
#include "Math/Interval.h"
#include "Math/InverseRotationMatrix.h"
#include "Math/Matrix.h"
#include "Math/MirrorMatrix.h"
#include "Math/NumericLimits.h"
#include "Math/OrientedBox.h"
#include "Math/OrthoMatrix.h"
#include "Math/PerspectiveMatrix.h"
#include "Math/Plane.h"
#include "Math/Quat.h"
#include "Math/QuatRotationTranslationMatrix.h"
#include "Math/Range.h"
#include "Math/RangeBound.h"
#include "Math/RangeSet.h"
#include "Math/Ray.h"
#include "Math/RotationAboutPointMatrix.h"
#include "Math/RotationMatrix.h"
#include "Math/RotationTranslationMatrix.h"
#include "Math/Rotator.h"
#include "Math/ScalarRegister.h"
#include "Math/ScaleMatrix.h"
#include "Math/ScaleRotationTranslationMatrix.h"
#include "Math/Sphere.h"
#include "Math/Transform.h"
#include "Math/TranslationMatrix.h"
#include "Math/TwoVectors.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/Vector2DHalf.h"
#include "Math/Vector4.h"
#include "Math/VectorRegister.h"
// IWYU pragma: end_exports
