// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "HAL/UnrealMemory.h"
#include <cmath>

/*=============================================================================
 *	Helpers:
 *============================================================================*/

struct VectorRegisterConstInit {};

 /**
 *	float4 vector register type, where the first float (X) is stored in the lowest 32 bits, and so on.
 */
struct alignas(16) VectorRegister4Float
{
	float	V[4];
};

/**
*	int32[4] vector register type, where the first int32 (X) is stored in the lowest 32 bits, and so on.
*/
struct VectorRegister4Int
{
	int32 V[4];
};


struct VectorRegister2Double
{
	double V[2];
};

/**
*	double[4] vector register type, where the first double (X) is stored in the lowest 64 bits, and so on.
*/
struct alignas(16) VectorRegister4Double
{
	union
	{
		struct
		{
			VectorRegister2Double XY;
			VectorRegister2Double ZW;
		};
		double V[4];
	};

	VectorRegister4Double() = default;

	FORCEINLINE VectorRegister4Double(const VectorRegister2Double& InXY, const VectorRegister2Double& InZW)
	{
		V[0] = InXY.V[0];
		V[1] = InXY.V[1];
		V[2] = InZW.V[0];
		V[3] = InZW.V[1];
	}

	FORCEINLINE constexpr VectorRegister4Double(VectorRegister2Double InXY, VectorRegister2Double InZW, VectorRegisterConstInit)
	: XY(InXY), ZW(InZW)
	{}

	// Construct from a vector of 4 floats
	FORCEINLINE VectorRegister4Double(const VectorRegister4Float& FloatVector)
	{
		V[0] = FloatVector.V[0];
		V[1] = FloatVector.V[1];
		V[2] = FloatVector.V[2];
		V[3] = FloatVector.V[3];
	}

	// Assign from a vector of 4 floats
	FORCEINLINE VectorRegister4Double& operator=(const VectorRegister4Float& From)
	{
		V[0] = From.V[0];
		V[1] = From.V[1];
		V[2] = From.V[2];
		V[3] = From.V[3];
		return *this;
	}
};

// Aliases
typedef VectorRegister4Int VectorRegister4i;
typedef VectorRegister4Float VectorRegister4f;
typedef VectorRegister4Double VectorRegister4d;
typedef VectorRegister2Double VectorRegister2d;

// Backwards compatibility
typedef VectorRegister4Double VectorRegister4;
typedef VectorRegister4 VectorRegister;
typedef VectorRegister4Int VectorRegisterInt;

typedef struct
{
	VectorRegister4Float val[4];
} VectorRegister4x4Float;


// Forward declarations
VectorRegister4Float VectorLoadAligned(const float* Ptr);
VectorRegister4Double VectorLoadAligned(const double* Ptr);
void VectorStoreAligned(const VectorRegister4Float& Vec, float* Ptr);
void VectorStoreAligned(const VectorRegister4Double& Vec, double* Dst);


// Helper for conveniently aligning a float array for extraction from VectorRegister4Float
struct alignas(alignof(VectorRegister4Float)) AlignedFloat4
{
	float V[4];

	FORCEINLINE AlignedFloat4(const VectorRegister4Float& Vec)
	{
		VectorStoreAligned(Vec, V);
	}

	FORCEINLINE float operator[](int32 Index) const { return V[Index]; }
	FORCEINLINE float& operator[](int32 Index) { return V[Index]; }

	FORCEINLINE VectorRegister4Float ToVectorRegister() const
	{
		return VectorLoadAligned(V);
	}
};


// Helper for conveniently aligning a double array for extraction from VectorRegister4Double
struct alignas(alignof(VectorRegister4Double)) AlignedDouble4
{
	double V[4];

	FORCEINLINE AlignedDouble4(const VectorRegister4Double& Vec)
	{
		VectorStoreAligned(Vec, V);
	}

	FORCEINLINE double operator[](int32 Index) const { return V[Index]; }
	FORCEINLINE double& operator[](int32 Index) { return V[Index]; }

	FORCEINLINE VectorRegister4Double ToVectorRegister() const
	{
		return VectorLoadAligned(V);
	}
};

typedef AlignedDouble4 AlignedRegister4;
#define VectorZeroVectorRegister() VectorZeroDouble()
#define VectorOneVectorRegister() VectorOneDouble()


#define DECLARE_VECTOR_REGISTER(X, Y, Z, W) MakeVectorRegister(X, Y, Z, W)


FORCEINLINE VectorRegister2Double MakeVectorRegister2Double(double X, double Y)
{
	VectorRegister2Double Result;
	Result.V[0] = X;
	Result.V[1] = Y;
	return Result;
}

/**
 * Returns a bitwise equivalent vector based on 4 DWORDs.
 *
 * @param X		1st uint32 component
 * @param Y		2nd uint32 component
 * @param Z		3rd uint32 component
 * @param W		4th uint32 component
 * @return		Bitwise equivalent vector with 4 floats
 */
FORCEINLINE VectorRegister4Float MakeVectorRegisterFloat( uint32 X, uint32 Y, uint32 Z, uint32 W )
{
	VectorRegister4Float Vec;
	((uint32&)Vec.V[0]) = X;
	((uint32&)Vec.V[1]) = Y;
	((uint32&)Vec.V[2]) = Z;
	((uint32&)Vec.V[3]) = W;
	return Vec;
}

FORCEINLINE VectorRegister4Double MakeVectorRegisterDouble(uint64 X, uint64 Y, uint64 Z, uint64 W)
{
	VectorRegister4Double Vec;
	((uint64&)Vec.V[0]) = X;
	((uint64&)Vec.V[1]) = Y;
	((uint64&)Vec.V[2]) = Z;
	((uint64&)Vec.V[3]) = W;
	return Vec;
}

FORCEINLINE VectorRegister4Float MakeVectorRegister(uint32 X, uint32 Y, uint32 Z, uint32 W)
{
	return MakeVectorRegisterFloat(X, Y, Z, W);
}

// Nicer aliases
FORCEINLINE VectorRegister4Float MakeVectorRegisterFloatMask(uint32 X, uint32 Y, uint32 Z, uint32 W)
{
	return MakeVectorRegisterFloat(X, Y, Z, W);
}

FORCEINLINE VectorRegister4Double MakeVectorRegisterDoubleMask(uint64 X, uint64 Y, uint64 Z, uint64 W)
{
	return MakeVectorRegisterDouble(X, Y, Z, W);
}

/**
 * Returns a vector based on 4 FLOATs.
 *
 * @param X		1st float component
 * @param Y		2nd float component
 * @param Z		3rd float component
 * @param W		4th float component
 * @return		Vector of the 4 FLOATs
 */
FORCEINLINE VectorRegister4Float MakeVectorRegisterFloat( float X, float Y, float Z, float W )
{
	VectorRegister4Float Vec;
	Vec.V[0] = X;
	Vec.V[1] = Y;
	Vec.V[2] = Z;
	Vec.V[3] = W;
	return Vec;
}

FORCEINLINE VectorRegister4Double MakeVectorRegisterDouble(double X, double Y, double Z, double W)
{
	VectorRegister4Double Vec;
	Vec.V[0] = X;
	Vec.V[1] = Y;
	Vec.V[2] = Z;
	Vec.V[3] = W;
	return Vec;
}

FORCEINLINE VectorRegister4Double MakeVectorRegisterDouble(const VectorRegister2Double& XY, const VectorRegister2Double& ZW)
{
	return VectorRegister4Double(XY, ZW);
}

FORCEINLINE VectorRegister4Float MakeVectorRegister(float X, float Y, float Z, float W)
{
	return MakeVectorRegisterFloat(X, Y, Z, W);
}

FORCEINLINE VectorRegister4Double MakeVectorRegister(double X, double Y, double Z, double W)
{
	return MakeVectorRegisterDouble(X, Y, Z, W);
}

// Make double register from float register
FORCEINLINE VectorRegister4Double MakeVectorRegisterDouble(const VectorRegister4Float& From)
{
	return VectorRegister4Double(From);
}

// Lossy conversion: double->float vector
FORCEINLINE VectorRegister4Float MakeVectorRegisterFloatFromDouble(const VectorRegister4Double& Vec4d)
{
	VectorRegister4Float Vec;
	Vec.V[0] = float(Vec4d.V[0]);
	Vec.V[1] = float(Vec4d.V[1]);
	Vec.V[2] = float(Vec4d.V[2]);
	Vec.V[3] = float(Vec4d.V[3]);
	return Vec;
}


/**
* Returns a vector based on 4 int32.
*
* @param X		1st int32 component
* @param Y		2nd int32 component
* @param Z		3rd int32 component
* @param W		4th int32 component
* @return		Vector of the 4 int32
*/
FORCEINLINE VectorRegister4Int MakeVectorRegisterInt(int32 X, int32 Y, int32 Z, int32 W)
{
	VectorRegister4Int Vec;
	((int32&)Vec.V[0]) = X;
	((int32&)Vec.V[1]) = Y;
	((int32&)Vec.V[2]) = Z;
	((int32&)Vec.V[3]) = W;
	return Vec;
}

FORCEINLINE constexpr VectorRegister4Int MakeVectorRegisterIntConstant(int32 X, int32 Y, int32 Z, int32 W)
{
    return VectorRegister4Int { X, Y, Z, W };
}

FORCEINLINE constexpr VectorRegister4Float MakeVectorRegisterFloatConstant(float X, float Y, float Z, float W)
{
	return VectorRegister4Float { X, Y, Z, W };
}

FORCEINLINE constexpr VectorRegister2Double MakeVectorRegister2DoubleConstant(double X, double Y)
{
	return VectorRegister2Double { X, Y };
}

FORCEINLINE VectorRegister4Int MakeVectorRegisterInt64(int64 X, int64 Y)
{
	VectorRegister4Int Vec;
	((int64&)Vec.V[0]) = X;
	((int64&)Vec.V[2]) = Y;
	return Vec;
}

/*=============================================================================
 *	Constants:
 *============================================================================*/

#include "Math/UnrealMathVectorConstants.h.inl"


/*=============================================================================
 *	Intrinsics:
 *============================================================================*/

/**
 * Returns a vector with all zeros.
 *
 * @return		MakeVectorRegister(0.0f, 0.0f, 0.0f, 0.0f)
 */
FORCEINLINE VectorRegister4Float VectorZeroFloat(void)
{
	return GlobalVectorConstants::FloatZero;
}

FORCEINLINE VectorRegister4Double VectorZeroDouble(void)
{
	return GlobalVectorConstants::DoubleZero;
}

/**
 * Returns a vector with all ones.
 *
 * @return		VectorRegister4Float(1.0f, 1.0f, 1.0f, 1.0f)
 */
FORCEINLINE VectorRegister4Float VectorOneFloat(void)
{
	return GlobalVectorConstants::FloatOne;
}

FORCEINLINE VectorRegister4Double VectorOneDouble(void)
{
	return GlobalVectorConstants::DoubleOne;
}

/**
 * Returns an component from a vector.
 *
 * @param Vec				Vector register
 * @param ComponentIndex	Which component to get, X=0, Y=1, Z=2, W=3
 * @return					The component as a float
 */
template <uint32 ComponentIndex>
FORCEINLINE float VectorGetComponentImpl(const VectorRegister4Float& Vec)
{
	return Vec.V[ComponentIndex];
}

FORCEINLINE float VectorGetComponentDynamic(const VectorRegister4Float& Vec, uint32 ComponentIndex)
{
	return Vec.V[ComponentIndex];
}

template <uint32 ComponentIndex>
FORCEINLINE double VectorGetComponentImpl(const VectorRegister4Double& Vec)
{
	return Vec.V[ComponentIndex];
}

FORCEINLINE double VectorGetComponentDynamic(const VectorRegister4Double& Vec, uint32 ComponentIndex)
{
	return Vec.V[ComponentIndex];
}

#define VectorGetComponent(Vec, ComponentIndex) VectorGetComponentImpl<ComponentIndex>(Vec)


/**
 * Loads 4 FLOATs from unaligned memory.
 *
 * @param Ptr	Unaligned memory pointer to the 4 FLOATs
 * @return		VectorRegister(Ptr[0], Ptr[1], Ptr[2], Ptr[3])
 */
FORCEINLINE VectorRegister4Float VectorLoad(const float* Ptr)
{
	return MakeVectorRegisterFloat(Ptr[0], Ptr[1], Ptr[2], Ptr[3]);
}

FORCEINLINE VectorRegister4Double VectorLoad(const double* Ptr)
{
	return MakeVectorRegisterDouble(Ptr[0], Ptr[1], Ptr[2], Ptr[3]);
}

/**
 * Loads 16 floats from unaligned memory into 4 vector registers.
 *
 * @param Ptr	Unaligned memory pointer to the 4 floats
 * @return		VectorRegister4x4Float containing 16 floats
 */
FORCEINLINE VectorRegister4x4Float VectorLoad16(const float* Ptr)
{
	VectorRegister4x4Float Result;
	Result.val[0] = VectorLoad(Ptr);
	Result.val[1] = VectorLoad(Ptr + 4);
	Result.val[2] = VectorLoad(Ptr + 8);
	Result.val[3] = VectorLoad(Ptr + 12);
	return Result;
}


/**
 * Loads 3 FLOATs from unaligned memory and sets W=0.
 *
 * @param Ptr	Unaligned memory pointer to the 3 FLOATs
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[2], 0)
 */
FORCEINLINE VectorRegister4Double VectorLoadFloat3(const double* Ptr)
{
	return MakeVectorRegisterDouble(Ptr[0], Ptr[1], Ptr[2], 0.0);
}


/**
 * Loads 3 FLOATs from unaligned memory and sets W=1.
 *
 * @param Ptr	Unaligned memory pointer to the 3 FLOATs
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[2], 1.0f)
 */
FORCEINLINE VectorRegister4Double VectorLoadFloat3_W1(const double* Ptr)
{
	return MakeVectorRegisterDouble(Ptr[0], Ptr[1], Ptr[2], 1.0);
}


/**
 * Loads 4 FLOATs from aligned memory.
 *
 * @param Ptr	Aligned memory pointer to the 4 FLOATs
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[2], Ptr[3])
 */
FORCEINLINE VectorRegister4Float VectorLoadAligned(const float* Ptr)
{
	return VectorLoad(Ptr);
}

FORCEINLINE VectorRegister4Double VectorLoadAligned(const double* Ptr)
{
	return VectorLoad(Ptr);
}


/**
 * Loads 1 float from unaligned memory and replicates it to all 4 elements.
 *
 * @param Ptr	Unaligned memory pointer to the float
 * @return		VectorRegister4Float(Ptr[0], Ptr[0], Ptr[0], Ptr[0])
 */
FORCEINLINE VectorRegister4Float VectorLoadFloat1(const float* Ptr)
{
	return MakeVectorRegisterFloat(Ptr[0], Ptr[0], Ptr[0], Ptr[0]);
}

FORCEINLINE VectorRegister4Double VectorLoadDouble1(const double* Ptr)
{
	return MakeVectorRegisterDouble(Ptr[0], Ptr[0], Ptr[0], Ptr[0]);
}


/**
 * Loads 2 floats from unaligned memory into X and Y and duplicates them in Z and W.
 *
 * @param Ptr	Unaligned memory pointer to the floats
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[0], Ptr[1])
 */
FORCEINLINE VectorRegister4Float VectorLoadFloat2(const float* Ptr)
{
	return MakeVectorRegisterFloat(Ptr[0], Ptr[1], Ptr[0], Ptr[1]);
}

FORCEINLINE VectorRegister4Double VectorLoadFloat2(const double* Ptr)
{
	return MakeVectorRegisterDouble(Ptr[0], Ptr[1], Ptr[0], Ptr[1]);
}

/**
 * Loads 4 unaligned floats - 2 from the first pointer, 2 from the second, and packs
 * them in to 1 vector.
 *
 * @param Ptr1	Unaligned memory pointer to the first 2 floats
 * @param Ptr2	Unaligned memory pointer to the second 2 floats
 * @return		VectorRegister(Ptr1[0], Ptr1[1], Ptr2[0], Ptr2[1])
 */
FORCEINLINE VectorRegister4Float VectorLoadTwoPairsFloat(const float* Ptr1, const float* Ptr2)
{
	return MakeVectorRegister(Ptr1[0], Ptr1[1], Ptr2[0], Ptr2[1]);
}
FORCEINLINE VectorRegister4Double VectorLoadTwoPairsFloat(const double* Ptr1, const double* Ptr2)
{
	return MakeVectorRegister(Ptr1[0], Ptr1[1], Ptr2[0], Ptr2[1]);
}

/**
 * Propagates passed in float to all registers
 *
 * @param F		Float to set
 * @return		VectorRegister4Float(F,F,F,F)
 */
FORCEINLINE VectorRegister4Float VectorSetFloat1(float F)
{
	return MakeVectorRegisterFloat(F, F, F, F);
}

FORCEINLINE VectorRegister4Double VectorSetFloat1(double D)
{
	return MakeVectorRegisterDouble(D, D, D, D);
}

/**
 * Stores a vector to aligned memory.
 *
 * @param Vec	Vector to store
 * @param Ptr	Aligned memory pointer
 */
FORCEINLINE void VectorStoreAligned(const VectorRegister4Float& Vec, float* Dst)
{
	FMemory::Memcpy(Dst, &Vec, 4 * sizeof(float));
}

FORCEINLINE void VectorStoreAligned(const VectorRegister4Double& Vec, double* Dst)
{
	FMemory::Memcpy(Dst, &Vec, 4 * sizeof(double));
}

/**
 * Performs non-temporal store of a vector to aligned memory without polluting the caches
 *
 * @param Vec	Vector to store
 * @param Ptr	Aligned memory pointer
 */
FORCEINLINE void VectorStoreAlignedStreamed(const VectorRegister4Float& Vec, float* Dst)
{
	VectorStoreAligned(Vec, Dst);
}

FORCEINLINE void VectorStoreAlignedStreamed(const VectorRegister4Double& Vec, double* Dst)
{
	VectorStoreAligned(Vec, Dst);
}

/**
 * Stores a vector to memory (aligned or unaligned).
 *
 * @param Vec	Vector to store
 * @param Ptr	Memory pointer
 */
FORCEINLINE void VectorStore(const VectorRegister4Float& Vec, float* Dst)
{
	FMemory::Memcpy(Dst, &(Vec), 4 * sizeof(float));
}

FORCEINLINE void VectorStore(const VectorRegister4Double& Vec, double* Dst)
{
	FMemory::Memcpy(Dst, &(Vec), 4 * sizeof(double));
}

/**
 * Stores 4 vectors to memory (aligned or unaligned).
 *
 * @param Vec	Vector to store
 * @param Ptr	Memory pointer
 */
FORCEINLINE void VectorStore16(const VectorRegister4x4Float& Vec, float* Dst)
{
	FMemory::Memcpy(Dst, &(Vec), 16 * sizeof(float));
}


/**
 * Stores the XYZ components of a vector to unaligned memory.
 *
 * @param Vec	Vector to store XYZ
 * @param Ptr	Unaligned memory pointer
 */
FORCEINLINE void VectorStoreFloat3(const VectorRegister4Float& Vec, float* Dst)
{
	FMemory::Memcpy(Dst, &(Vec), 3 * sizeof(float));
}

FORCEINLINE void VectorStoreFloat3(const VectorRegister4Double& Vec, double* Dst)
{
	FMemory::Memcpy(Dst, &(Vec), 3 * sizeof(double));
}

/**
 * Stores the X component of a vector to unaligned memory.
 *
 * @param Vec	Vector to store X
 * @param Ptr	Unaligned memory pointer
 */
FORCEINLINE void VectorStoreFloat1(const VectorRegister4Float& Vec, float* Dst)
{
	FMemory::Memcpy(Dst, &(Vec), 1 * sizeof(float));
}

FORCEINLINE void VectorStoreFloat1(const VectorRegister4Double& Vec, double* Dst)
{
	FMemory::Memcpy(Dst, &(Vec), 1 * sizeof(double));
}

/**
 * Replicates one element into all four elements and returns the new vector.
 *
 * @param Vec			Source vector
 * @param ElementIndex	Index (0-3) of the element to replicate
 * @return				VectorRegister( Vec[ElementIndex], Vec[ElementIndex], Vec[ElementIndex], Vec[ElementIndex] )
 */
#define VectorReplicate( Vec, ElementIndex )	MakeVectorRegister( (Vec).V[ElementIndex], (Vec).V[ElementIndex], (Vec).V[ElementIndex], (Vec).V[ElementIndex] )

/**
 * Swizzles the 4 components of a vector and returns the result.
 *
 * @param Vec		Source vector
 * @param X			Index for which component to use for X (literal 0-3)
 * @param Y			Index for which component to use for Y (literal 0-3)
 * @param Z			Index for which component to use for Z (literal 0-3)
 * @param W			Index for which component to use for W (literal 0-3)
 * @return			The swizzled vector
 */
#define VectorSwizzle( Vec, X, Y, Z, W )	MakeVectorRegister( (Vec).V[X], (Vec).V[Y], (Vec).V[Z], (Vec).V[W] )

/**
 * Creates a vector through selecting two components from each vector via a shuffle mask. 
 *
 * @param Vec1		Source vector1
 * @param Vec2		Source vector2
 * @param X			Index for which component of Vector1 to use for X (literal 0-3)
 * @param Y			Index for which component to Vector1 to use for Y (literal 0-3)
 * @param Z			Index for which component to Vector2 to use for Z (literal 0-3)
 * @param W			Index for which component to Vector2 to use for W (literal 0-3)
 * @return			The swizzled vector
 */
#define VectorShuffle( Vec1, Vec2, X, Y, Z, W )	MakeVectorRegister( (Vec1).V[X], (Vec1).V[Y], (Vec2).V[Z], (Vec2).V[W] )


/**
 * Returns the absolute value (component-wise).
 *
 * @param Vec			Source vector
 * @return				VectorRegister( abs(Vec.x), abs(Vec.y), abs(Vec.z), abs(Vec.w) )
 */
FORCEINLINE VectorRegister4Float VectorAbs(const VectorRegister4Float& Vec)
{
	VectorRegister4Float Vec2;
	Vec2.V[0] = FMath::Abs(Vec.V[0]);
	Vec2.V[1] = FMath::Abs(Vec.V[1]);
	Vec2.V[2] = FMath::Abs(Vec.V[2]);
	Vec2.V[3] = FMath::Abs(Vec.V[3]);
	return Vec2;
}

FORCEINLINE VectorRegister4Double VectorAbs(const VectorRegister4Double& Vec)
{
	VectorRegister4Double Vec2;
	Vec2.V[0] = FMath::Abs(Vec.V[0]);
	Vec2.V[1] = FMath::Abs(Vec.V[1]);
	Vec2.V[2] = FMath::Abs(Vec.V[2]);
	Vec2.V[3] = FMath::Abs(Vec.V[3]);
	return Vec2;
}

/**
 * Returns the negated value (component-wise).
 *
 * @param Vec			Source vector
 * @return				VectorRegister( -Vec.x, -Vec.y, -Vec.z, -Vec.w )
 */
FORCEINLINE VectorRegister4Float VectorNegate(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(-(Vec).V[0], -(Vec).V[1], -(Vec).V[2], -(Vec).V[3]);
}

FORCEINLINE VectorRegister4Double VectorNegate(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(-(Vec).V[0], -(Vec).V[1], -(Vec).V[2], -(Vec).V[3]);
}

/**
 * Adds two vectors (component-wise) and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister( Vec1.x+Vec2.x, Vec1.y+Vec2.y, Vec1.z+Vec2.z, Vec1.w+Vec2.w )
 */
FORCEINLINE VectorRegister4Float VectorAdd(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	VectorRegister4Float Vec;
	Vec.V[0] = Vec1.V[0] + Vec2.V[0];
	Vec.V[1] = Vec1.V[1] + Vec2.V[1];
	Vec.V[2] = Vec1.V[2] + Vec2.V[2];
	Vec.V[3] = Vec1.V[3] + Vec2.V[3];
	return Vec;
}

FORCEINLINE VectorRegister4Double VectorAdd(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	VectorRegister4Double Vec;
	Vec.V[0] = Vec1.V[0] + Vec2.V[0];
	Vec.V[1] = Vec1.V[1] + Vec2.V[1];
	Vec.V[2] = Vec1.V[2] + Vec2.V[2];
	Vec.V[3] = Vec1.V[3] + Vec2.V[3];
	return Vec;
}

/**
 * Subtracts a vector from another (component-wise) and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister( Vec1.x-Vec2.x, Vec1.y-Vec2.y, Vec1.z-Vec2.z, Vec1.w-Vec2.w )
 */
FORCEINLINE VectorRegister4Float VectorSubtract(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	VectorRegister4Float Vec;
	Vec.V[0] = Vec1.V[0] - Vec2.V[0];
	Vec.V[1] = Vec1.V[1] - Vec2.V[1];
	Vec.V[2] = Vec1.V[2] - Vec2.V[2];
	Vec.V[3] = Vec1.V[3] - Vec2.V[3];
	return Vec;
}

FORCEINLINE VectorRegister4Double VectorSubtract(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	VectorRegister4Double Vec;
	Vec.V[0] = Vec1.V[0] - Vec2.V[0];
	Vec.V[1] = Vec1.V[1] - Vec2.V[1];
	Vec.V[2] = Vec1.V[2] - Vec2.V[2];
	Vec.V[3] = Vec1.V[3] - Vec2.V[3];
	return Vec;
}

/**
 * Multiplies two vectors (component-wise) and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister( Vec1.x*Vec2.x, Vec1.y*Vec2.y, Vec1.z*Vec2.z, Vec1.w*Vec2.w )
 */
FORCEINLINE VectorRegister4Float VectorMultiply(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	VectorRegister4Float Vec;
	Vec.V[0] = Vec1.V[0] * Vec2.V[0];
	Vec.V[1] = Vec1.V[1] * Vec2.V[1];
	Vec.V[2] = Vec1.V[2] * Vec2.V[2];
	Vec.V[3] = Vec1.V[3] * Vec2.V[3];
	return Vec;
}

FORCEINLINE VectorRegister4Double VectorMultiply(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	VectorRegister4Double Vec;
	Vec.V[0] = Vec1.V[0] * Vec2.V[0];
	Vec.V[1] = Vec1.V[1] * Vec2.V[1];
	Vec.V[2] = Vec1.V[2] * Vec2.V[2];
	Vec.V[3] = Vec1.V[3] * Vec2.V[3];
	return Vec;
}

/**
 * Multiplies two vectors (component-wise), adds in the third vector and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @param Vec3	3rd vector
 * @return		VectorRegister( Vec1.x*Vec2.x + Vec3.x, Vec1.y*Vec2.y + Vec3.y, Vec1.z*Vec2.z + Vec3.z, Vec1.w*Vec2.w + Vec3.w )
 */
FORCEINLINE VectorRegister4Float VectorMultiplyAdd(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2, const VectorRegister4Float& Vec3)
{
	VectorRegister4Float Vec;
	Vec.V[0] = Vec1.V[0] * Vec2.V[0] + Vec3.V[0];
	Vec.V[1] = Vec1.V[1] * Vec2.V[1] + Vec3.V[1];
	Vec.V[2] = Vec1.V[2] * Vec2.V[2] + Vec3.V[2];
	Vec.V[3] = Vec1.V[3] * Vec2.V[3] + Vec3.V[3];
	return Vec;
}

FORCEINLINE VectorRegister4Double VectorMultiplyAdd(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2, const VectorRegister4Double& Vec3)
{
	VectorRegister4Double Vec;
	Vec.V[0] = Vec1.V[0] * Vec2.V[0] + Vec3.V[0];
	Vec.V[1] = Vec1.V[1] * Vec2.V[1] + Vec3.V[1];
	Vec.V[2] = Vec1.V[2] * Vec2.V[2] + Vec3.V[2];
	Vec.V[3] = Vec1.V[3] * Vec2.V[3] + Vec3.V[3];
	return Vec;
}


/**
 * Multiplies two vectors (component-wise), negates the results and adds it to the third vector i.e. -AB + C = C - AB
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @param Vec3	3rd vector
 * @return		VectorRegister( Vec3.x - Vec1.x*Vec2.x, Vec3.y - Vec1.y*Vec2.y, Vec3.z - Vec1.z*Vec2.z, Vec3.w - Vec1.w*Vec2.w )
 */
FORCEINLINE VectorRegister4Float VectorNegateMultiplyAdd(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2, const VectorRegister4Float& Vec3)
{
	return VectorSubtract(Vec3, VectorMultiply(Vec1, Vec2));
}

FORCEINLINE VectorRegister4Double VectorNegateMultiplyAdd(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2, const VectorRegister4Double& Vec3)
{
	return VectorSubtract(Vec3, VectorMultiply(Vec1, Vec2));
}



/**
* Divides two vectors (component-wise) and returns the result.
*
* @param Vec1	1st vector
* @param Vec2	2nd vector
* @return		VectorRegister( Vec1.x/Vec2.x, Vec1.y/Vec2.y, Vec1.z/Vec2.z, Vec1.w/Vec2.w )
*/
FORCEINLINE VectorRegister4Float VectorDivide(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	VectorRegister4Float Vec;
	Vec.V[0] = Vec1.V[0] / Vec2.V[0];
	Vec.V[1] = Vec1.V[1] / Vec2.V[1];
	Vec.V[2] = Vec1.V[2] / Vec2.V[2];
	Vec.V[3] = Vec1.V[3] / Vec2.V[3];
	return Vec;
}

FORCEINLINE VectorRegister4Double VectorDivide(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	VectorRegister4Double Vec;
	Vec.V[0] = Vec1.V[0] / Vec2.V[0];
	Vec.V[1] = Vec1.V[1] / Vec2.V[1];
	Vec.V[2] = Vec1.V[2] / Vec2.V[2];
	Vec.V[3] = Vec1.V[3] / Vec2.V[3];
	return Vec;
}

/**
 * Calculates the dot3 product of two vectors and returns a scalar value.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		d = dot3(Vec1.xyz, Vec2.xyz)
 */
FORCEINLINE float VectorDot3Scalar(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	float D = Vec1.V[0] * Vec2.V[0] + Vec1.V[1] * Vec2.V[1] + Vec1.V[2] * Vec2.V[2];
	return D;
}

FORCEINLINE double VectorDot3Scalar(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	double D = Vec1.V[0] * Vec2.V[0] + Vec1.V[1] * Vec2.V[1] + Vec1.V[2] * Vec2.V[2];
	return D;
}

/**
 * Calculates the dot3 product of two vectors and returns a vector with the result in all 4 components.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		d = dot3(Vec1.xyz, Vec2.xyz), VectorRegister( d, d, d, d )
 */
FORCEINLINE VectorRegister4Float VectorDot3(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	float D = VectorDot3Scalar(Vec1, Vec2);
	VectorRegister4Float Vec = MakeVectorRegisterFloat(D, D, D, D);
	return Vec;
}

FORCEINLINE VectorRegister4Double VectorDot3(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	double D = VectorDot3Scalar(Vec1, Vec2);
	VectorRegister4Double Vec = MakeVectorRegisterDouble(D, D, D, D);
	return Vec;
}

/**
 * Calculates the dot4 product of two vectors and returns a vector with the result in all 4 components.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		d = dot4(Vec1.xyzw, Vec2.xyzw), VectorRegister( d, d, d, d )
 */
FORCEINLINE VectorRegister4Float VectorDot4(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	float D = Vec1.V[0] * Vec2.V[0] + Vec1.V[1] * Vec2.V[1] + Vec1.V[2] * Vec2.V[2] + Vec1.V[3] * Vec2.V[3];
	VectorRegister4Float Vec = MakeVectorRegisterFloat(D, D, D, D);
	return Vec;
}

FORCEINLINE VectorRegister4Double VectorDot4(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	double D = Vec1.V[0] * Vec2.V[0] + Vec1.V[1] * Vec2.V[1] + Vec1.V[2] * Vec2.V[2] + Vec1.V[3] * Vec2.V[3];
	VectorRegister4Double Vec = MakeVectorRegisterDouble(D, D, D, D);
	return Vec;
}


/**
 * Creates a four-part mask based on component-wise == compares of the input vectors
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister( Vec1.x == Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
 */
FORCEINLINE VectorRegister4Float VectorCompareEQ(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	return MakeVectorRegisterFloat(
		uint32(Vec1.V[0] == Vec2.V[0] && !std::isunordered(Vec1.V[0], Vec2.V[0]) ? 0xFFFFFFFF : 0),
		uint32(Vec1.V[1] == Vec2.V[1] && !std::isunordered(Vec1.V[1], Vec2.V[1]) ? 0xFFFFFFFF : 0),
		uint32(Vec1.V[2] == Vec2.V[2] && !std::isunordered(Vec1.V[2], Vec2.V[2]) ? 0xFFFFFFFF : 0),
		uint32(Vec1.V[3] == Vec2.V[3] && !std::isunordered(Vec1.V[3], Vec2.V[3]) ? 0xFFFFFFFF : 0));
}

FORCEINLINE VectorRegister4Double VectorCompareEQ(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	return MakeVectorRegisterDouble(
		uint64(Vec1.V[0] == Vec2.V[0] && !std::isunordered(Vec1.V[0], Vec2.V[0]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(Vec1.V[1] == Vec2.V[1] && !std::isunordered(Vec1.V[1], Vec2.V[1]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(Vec1.V[2] == Vec2.V[2] && !std::isunordered(Vec1.V[2], Vec2.V[2]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(Vec1.V[3] == Vec2.V[3] && !std::isunordered(Vec1.V[3], Vec2.V[3]) ? 0xFFFFFFFFFFFFFFFF : 0));
}

/**
 * Creates a four-part mask based on component-wise != compares of the input vectors
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister( Vec1.x != Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
 */
FORCEINLINE VectorRegister4Float VectorCompareNE(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	return MakeVectorRegisterFloat(
		uint32(Vec1.V[0] != Vec2.V[0] || std::isunordered(Vec1.V[0], Vec2.V[0]) ? 0xFFFFFFFF : 0),
		uint32(Vec1.V[1] != Vec2.V[1] || std::isunordered(Vec1.V[1], Vec2.V[1]) ? 0xFFFFFFFF : 0),
		uint32(Vec1.V[2] != Vec2.V[2] || std::isunordered(Vec1.V[2], Vec2.V[2]) ? 0xFFFFFFFF : 0),
		uint32(Vec1.V[3] != Vec2.V[3] || std::isunordered(Vec1.V[3], Vec2.V[3]) ? 0xFFFFFFFF : 0));
}

FORCEINLINE VectorRegister4Double VectorCompareNE(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	return MakeVectorRegisterDouble(
		uint64(Vec1.V[0] != Vec2.V[0] || std::isunordered(Vec1.V[0], Vec2.V[0]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(Vec1.V[1] != Vec2.V[1] || std::isunordered(Vec1.V[1], Vec2.V[1]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(Vec1.V[2] != Vec2.V[2] || std::isunordered(Vec1.V[2], Vec2.V[2]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(Vec1.V[3] != Vec2.V[3] || std::isunordered(Vec1.V[3], Vec2.V[3]) ? 0xFFFFFFFFFFFFFFFF : 0));
}

/**
 * Creates a four-part mask based on component-wise > compares of the input vectors
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister( Vec1.x > Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
 */

FORCEINLINE VectorRegister4Float VectorCompareGT(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	return MakeVectorRegisterFloat(
		uint32(std::isgreater(Vec1.V[0], Vec2.V[0]) ? 0xFFFFFFFF : 0),
		uint32(std::isgreater(Vec1.V[1], Vec2.V[1]) ? 0xFFFFFFFF : 0),
		uint32(std::isgreater(Vec1.V[2], Vec2.V[2]) ? 0xFFFFFFFF : 0),
		uint32(std::isgreater(Vec1.V[3], Vec2.V[3]) ? 0xFFFFFFFF : 0));
}

FORCEINLINE VectorRegister4Double VectorCompareGT(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	return MakeVectorRegisterDouble(
		uint64(std::isgreater(Vec1.V[0], Vec2.V[0]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(std::isgreater(Vec1.V[1], Vec2.V[1]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(std::isgreater(Vec1.V[2], Vec2.V[2]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(std::isgreater(Vec1.V[3], Vec2.V[3]) ? 0xFFFFFFFFFFFFFFFF : 0));
}

/**
 * Creates a four-part mask based on component-wise >= compares of the input vectors
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister( Vec1.x >= Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
 */

FORCEINLINE VectorRegister4Float VectorCompareGE(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	return MakeVectorRegisterFloat(
		uint32(std::isgreaterequal(Vec1.V[0], Vec2.V[0]) ? 0xFFFFFFFF : 0),
		uint32(std::isgreaterequal(Vec1.V[1], Vec2.V[1]) ? 0xFFFFFFFF : 0),
		uint32(std::isgreaterequal(Vec1.V[2], Vec2.V[2]) ? 0xFFFFFFFF : 0),
		uint32(std::isgreaterequal(Vec1.V[3], Vec2.V[3]) ? 0xFFFFFFFF : 0));
}

FORCEINLINE VectorRegister4Double VectorCompareGE(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	return MakeVectorRegisterDouble(
		uint64(std::isgreaterequal(Vec1.V[0], Vec2.V[0]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(std::isgreaterequal(Vec1.V[1], Vec2.V[1]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(std::isgreaterequal(Vec1.V[2], Vec2.V[2]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(std::isgreaterequal(Vec1.V[3], Vec2.V[3]) ? 0xFFFFFFFFFFFFFFFF : 0));
}

/**
* Creates a four-part mask based on component-wise < compares of the input vectors
*
* @param Vec1	1st vector
* @param Vec2	2nd vector
* @return		VectorRegister( Vec1.x < Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
*/
FORCEINLINE VectorRegister4Float VectorCompareLT(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	return MakeVectorRegisterFloat(
		uint32(std::isless(Vec1.V[0], Vec2.V[0]) ? 0xFFFFFFFF : 0),
		uint32(std::isless(Vec1.V[1], Vec2.V[1]) ? 0xFFFFFFFF : 0),
		uint32(std::isless(Vec1.V[2], Vec2.V[2]) ? 0xFFFFFFFF : 0),
		uint32(std::isless(Vec1.V[3], Vec2.V[3]) ? 0xFFFFFFFF : 0));
}

FORCEINLINE VectorRegister4Double VectorCompareLT(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	return MakeVectorRegisterDouble(
		uint64(std::isless(Vec1.V[0], Vec2.V[0]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(std::isless(Vec1.V[1], Vec2.V[1]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(std::isless(Vec1.V[2], Vec2.V[2]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(std::isless(Vec1.V[3], Vec2.V[3]) ? 0xFFFFFFFFFFFFFFFF : 0));
}

/**
* Creates a four-part mask based on component-wise <= compares of the input vectors
*
* @param Vec1	1st vector
* @param Vec2	2nd vector
* @return		VectorRegister( Vec1.x <= Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
*/
FORCEINLINE VectorRegister4Float VectorCompareLE(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	return MakeVectorRegisterFloat(
		uint32(std::islessequal(Vec1.V[0], Vec2.V[0]) ? 0xFFFFFFFF : 0),
		uint32(std::islessequal(Vec1.V[1], Vec2.V[1]) ? 0xFFFFFFFF : 0),
		uint32(std::islessequal(Vec1.V[2], Vec2.V[2]) ? 0xFFFFFFFF : 0),
		uint32(std::islessequal(Vec1.V[3], Vec2.V[3]) ? 0xFFFFFFFF : 0));
}

FORCEINLINE VectorRegister4Double VectorCompareLE(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	return MakeVectorRegisterDouble(
		uint64(std::islessequal(Vec1.V[0], Vec2.V[0]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(std::islessequal(Vec1.V[1], Vec2.V[1]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(std::islessequal(Vec1.V[2], Vec2.V[2]) ? 0xFFFFFFFFFFFFFFFF : 0),
		uint64(std::islessequal(Vec1.V[3], Vec2.V[3]) ? 0xFFFFFFFFFFFFFFFF : 0));
}


/**
 * Returns an integer bit-mask (0x00 - 0x0f) based on the sign-bit for each component in a vector.
 *
 * @param VecMask		Vector
 * @return				Bit 0 = sign(VecMask.x), Bit 1 = sign(VecMask.y), Bit 2 = sign(VecMask.z), Bit 3 = sign(VecMask.w)
 */
FORCEINLINE int32 VectorMaskBits(const VectorRegister4Float& Vec1)
{
	uint32* V1 = (uint32*)(&(Vec1.V[0]));

	return	((V1[0] >> 31)) |
			((V1[1] >> 30) & 2) |
			((V1[2] >> 29) & 4) |
			((V1[3] >> 28) & 8);
}

FORCEINLINE int32 VectorMaskBits(const VectorRegister4Double& Vec1)
{
	uint64* V1 = (uint64*)(&(Vec1.V[0]));

	return	((V1[0] >> 63)) |
			((V1[1] >> 62) & 2) |
			((V1[2] >> 61) & 4) |
			((V1[3] >> 60) & 8);
}

/**
 * Does a bitwise vector selection based on a mask (e.g., created from VectorCompareXX)
 *
 * @param Mask  Mask (when 1: use the corresponding bit from Vec1 otherwise from Vec2)
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister( for each bit i: Mask[i] ? Vec1[i] : Vec2[i] )
 *
 */

FORCEINLINE VectorRegister4Float VectorSelect(const VectorRegister4Float& Mask, const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	uint32* V1 = (uint32*)(&(Vec1.V[0]));
	uint32* V2 = (uint32*)(&(Vec2.V[0]));
	uint32* M  = (uint32*)(&(Mask.V[0]));

	return MakeVectorRegisterFloat(
		V2[0] ^ (M[0] & (V2[0] ^ V1[0])),
		V2[1] ^ (M[1] & (V2[1] ^ V1[1])),
		V2[2] ^ (M[2] & (V2[2] ^ V1[2])),
		V2[3] ^ (M[3] & (V2[3] ^ V1[3]))
	);
}

FORCEINLINE VectorRegister4Double VectorSelect(const VectorRegister4Double& Mask, const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	uint64* V1 = (uint64*)(&(Vec1.V[0]));
	uint64* V2 = (uint64*)(&(Vec2.V[0]));
	uint64* M  = (uint64*)(&(Mask.V[0]));

	return MakeVectorRegisterDouble(
		V2[0] ^ (M[0] & (V2[0] ^ V1[0])),
		V2[1] ^ (M[1] & (V2[1] ^ V1[1])),
		V2[2] ^ (M[2] & (V2[2] ^ V1[2])),
		V2[3] ^ (M[3] & (V2[3] ^ V1[3]))
	);
}

/**
 * Combines two vectors using bitwise OR (treating each vector as a 128 bit field)
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister( for each bit i: Vec1[i] | Vec2[i] )
 */
FORCEINLINE VectorRegister4Float VectorBitwiseOr(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	return MakeVectorRegisterFloat(
		uint32(((uint32*)(Vec1.V))[0] | ((uint32*)(Vec2.V))[0]),
		uint32(((uint32*)(Vec1.V))[1] | ((uint32*)(Vec2.V))[1]),
		uint32(((uint32*)(Vec1.V))[2] | ((uint32*)(Vec2.V))[2]),
		uint32(((uint32*)(Vec1.V))[3] | ((uint32*)(Vec2.V))[3]));
}

FORCEINLINE VectorRegister4Double VectorBitwiseOr(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	return MakeVectorRegisterDouble(
		uint64(((uint64*)(Vec1.V))[0] | ((uint64*)(Vec2.V))[0]),
		uint64(((uint64*)(Vec1.V))[1] | ((uint64*)(Vec2.V))[1]),
		uint64(((uint64*)(Vec1.V))[2] | ((uint64*)(Vec2.V))[2]),
		uint64(((uint64*)(Vec1.V))[3] | ((uint64*)(Vec2.V))[3]));
}

/**
 * Combines two vectors using bitwise AND (treating each vector as a 128 bit field)
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister( for each bit i: Vec1[i] & Vec2[i] )
 */
FORCEINLINE VectorRegister4Float VectorBitwiseAnd(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	return MakeVectorRegisterFloat(
		uint32(((uint32*)(Vec1.V))[0] & ((uint32*)(Vec2.V))[0]),
		uint32(((uint32*)(Vec1.V))[1] & ((uint32*)(Vec2.V))[1]),
		uint32(((uint32*)(Vec1.V))[2] & ((uint32*)(Vec2.V))[2]),
		uint32(((uint32*)(Vec1.V))[3] & ((uint32*)(Vec2.V))[3]));
}

FORCEINLINE VectorRegister4Double VectorBitwiseAnd(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	return MakeVectorRegisterDouble(
		uint64(((uint64*)(Vec1.V))[0] & ((uint64*)(Vec2.V))[0]),
		uint64(((uint64*)(Vec1.V))[1] & ((uint64*)(Vec2.V))[1]),
		uint64(((uint64*)(Vec1.V))[2] & ((uint64*)(Vec2.V))[2]),
		uint64(((uint64*)(Vec1.V))[3] & ((uint64*)(Vec2.V))[3]));
}

/**
 * Combines two vectors using bitwise XOR (treating each vector as a 128 bit field)
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister( for each bit i: Vec1[i] ^ Vec2[i] )
 */
FORCEINLINE VectorRegister4Float VectorBitwiseXor(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	return MakeVectorRegisterFloat(
		uint32(((uint32*)(Vec1.V))[0] ^ ((uint32*)(Vec2.V))[0]),
		uint32(((uint32*)(Vec1.V))[1] ^ ((uint32*)(Vec2.V))[1]),
		uint32(((uint32*)(Vec1.V))[2] ^ ((uint32*)(Vec2.V))[2]),
		uint32(((uint32*)(Vec1.V))[3] ^ ((uint32*)(Vec2.V))[3]));
}

FORCEINLINE VectorRegister4Double VectorBitwiseXor(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	return MakeVectorRegisterDouble(
		uint64(((uint64*)(Vec1.V))[0] ^ ((uint64*)(Vec2.V))[0]),
		uint64(((uint64*)(Vec1.V))[1] ^ ((uint64*)(Vec2.V))[1]),
		uint64(((uint64*)(Vec1.V))[2] ^ ((uint64*)(Vec2.V))[2]),
		uint64(((uint64*)(Vec1.V))[3] ^ ((uint64*)(Vec2.V))[3]));
}


/**
 * Calculates the cross product of two vectors (XYZ components). W of the input should be 0, and will remain 0.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		cross(Vec1.xyz, Vec2.xyz). W of the input should be 0, and will remain 0.
 */
FORCEINLINE VectorRegister4Float VectorCross(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	VectorRegister4Float Vec;
	Vec.V[0] = Vec1.V[1] * Vec2.V[2] - Vec1.V[2] * Vec2.V[1];
	Vec.V[1] = Vec1.V[2] * Vec2.V[0] - Vec1.V[0] * Vec2.V[2];
	Vec.V[2] = Vec1.V[0] * Vec2.V[1] - Vec1.V[1] * Vec2.V[0];
	Vec.V[3] = 0.0f;
	return Vec;
}

FORCEINLINE VectorRegister4Double VectorCross(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	VectorRegister4Double Vec;
	Vec.V[0] = Vec1.V[1] * Vec2.V[2] - Vec1.V[2] * Vec2.V[1];
	Vec.V[1] = Vec1.V[2] * Vec2.V[0] - Vec1.V[0] * Vec2.V[2];
	Vec.V[2] = Vec1.V[0] * Vec2.V[1] - Vec1.V[1] * Vec2.V[0];
	Vec.V[3] = 0.0;
	return Vec;
}

/**
 * Calculates x raised to the power of y (component-wise).
 *
 * @param Base		Base vector
 * @param Exponent	Exponent vector
 * @return			VectorRegister( Base.x^Exponent.x, Base.y^Exponent.y, Base.z^Exponent.z, Base.w^Exponent.w )
 */
FORCEINLINE VectorRegister4Float VectorPow(const VectorRegister4Float& Base, const VectorRegister4Float& Exponent)
{
	VectorRegister4Float Vec;
	Vec.V[0] = FMath::Pow(Base.V[0], Exponent.V[0]);
	Vec.V[1] = FMath::Pow(Base.V[1], Exponent.V[1]);
	Vec.V[2] = FMath::Pow(Base.V[2], Exponent.V[2]);
	Vec.V[3] = FMath::Pow(Base.V[3], Exponent.V[3]);
	return Vec;
}

FORCEINLINE VectorRegister4Double VectorPow(const VectorRegister4Double& Base, const VectorRegister4Double& Exponent)
{
	VectorRegister4Double Vec;
	Vec.V[0] = FMath::Pow(Base.V[0], Exponent.V[0]);
	Vec.V[1] = FMath::Pow(Base.V[1], Exponent.V[1]);
	Vec.V[2] = FMath::Pow(Base.V[2], Exponent.V[2]);
	Vec.V[3] = FMath::Pow(Base.V[3], Exponent.V[3]);
	return Vec;
}

FORCEINLINE VectorRegister4Float VectorSqrt(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::Sqrt(Vec.V[0]), FMath::Sqrt(Vec.V[1]), FMath::Sqrt(Vec.V[2]), FMath::Sqrt(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorSqrt(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::Sqrt(Vec.V[0]), FMath::Sqrt(Vec.V[1]), FMath::Sqrt(Vec.V[2]), FMath::Sqrt(Vec.V[3]));
}

/**
* Returns an estimate of 1/sqrt(c) for each component of the vector
*
* @param Vector		Vector
* @return			VectorRegister(1/sqrt(t), 1/sqrt(t), 1/sqrt(t), 1/sqrt(t))
*/
FORCEINLINE VectorRegister4Float VectorReciprocalSqrt(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(1.0f / FMath::Sqrt(Vec.V[0]), 1.0f / FMath::Sqrt(Vec.V[1]), 1.0f / FMath::Sqrt(Vec.V[2]), 1.0f / FMath::Sqrt(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorReciprocalSqrt(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(1.0 / FMath::Sqrt(Vec.V[0]), 1.0 / FMath::Sqrt(Vec.V[1]), 1.0 / FMath::Sqrt(Vec.V[2]), 1.0 / FMath::Sqrt(Vec.V[3]));
}

/**
* Returns an estimate of 1/sqrt(c) for each component of the vector
*
* @param Vector		Vector 
* @return			VectorRegister(1/sqrt(t), 1/sqrt(t), 1/sqrt(t), 1/sqrt(t))
*/
FORCEINLINE VectorRegister4Float VectorReciprocalSqrtEstimate(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::InvSqrtEst(Vec.V[0]), FMath::InvSqrtEst(Vec.V[1]), FMath::InvSqrtEst(Vec.V[2]), FMath::InvSqrtEst(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorReciprocalSqrtEstimate(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::InvSqrtEst(Vec.V[0]), FMath::InvSqrtEst(Vec.V[1]), FMath::InvSqrtEst(Vec.V[2]), FMath::InvSqrtEst(Vec.V[3]));
}

/**
 * Computes the reciprocal of a vector (component-wise) and returns the result.
 *
 * @param Vec	1st vector
 * @return		VectorRegister4Float( 1.0f / Vec.x, 1.0f / Vec.y, 1.0f / Vec.z, 1.0f / Vec.w )
 */
FORCEINLINE VectorRegister4Float VectorReciprocal(const VectorRegister4Float& Vec)
{
	return VectorDivide(GlobalVectorConstants::FloatOne, Vec);
}

FORCEINLINE VectorRegister4Double VectorReciprocal(const VectorRegister4Double& Vec)
{
	return VectorDivide(GlobalVectorConstants::DoubleOne, Vec);
}

/**
 * Computes an estimate of the reciprocal of a vector (component-wise) and returns the result.
 *
 * @param Vec	1st vector
 * @return		VectorRegister( (Estimate) 1.0f / Vec.x, (Estimate) 1.0f / Vec.y, (Estimate) 1.0f / Vec.z, (Estimate) 1.0f / Vec.w )
 */
FORCEINLINE VectorRegister4Float VectorReciprocalEstimate(const VectorRegister4Float& Vec)
{
	return VectorReciprocal(Vec);
}

FORCEINLINE VectorRegister4Double VectorReciprocalEstimate(const VectorRegister4Double& Vec)
{
	return VectorReciprocal(Vec);
}

/**
* Return Reciprocal Length of the vector
*
* @param Vec		Vector 
* @return			VectorRegister(rlen, rlen, rlen, rlen) when rlen = 1/sqrt(dot4(V))
*/
FORCEINLINE VectorRegister4Float VectorReciprocalLen( const VectorRegister4Float& Vec)
{
	VectorRegister4Float Len = VectorDot4(Vec, Vec);
	float rlen = 1.0f / FMath::Sqrt(Len.V[0]);
	
	VectorRegister4Float Result;
	Result.V[0] = rlen;
	Result.V[1] = rlen;
	Result.V[2] = rlen;
	Result.V[3] = rlen;
	return Result;
}

FORCEINLINE VectorRegister4Double VectorReciprocalLen(const VectorRegister4Double& Vec)
{
	VectorRegister4Double Len = VectorDot4(Vec, Vec);
	double rlen = 1.0 / FMath::Sqrt(Len.V[0]);

	VectorRegister4Double Result;
	Result.V[0] = rlen;
	Result.V[1] = rlen;
	Result.V[2] = rlen;
	Result.V[3] = rlen;
	return Result;
}

/**
 * Return Reciprocal Length of the vector (estimate)
 *
 * @param Vector		Vector
 * @return			VectorRegister4Float(rlen, rlen, rlen, rlen) when rlen = 1/sqrt(dot4(V))
 */
FORCEINLINE VectorRegister4Float VectorReciprocalLenEstimate(const VectorRegister4Float& Vector)
{
	return VectorReciprocalLen(Vector);
}

FORCEINLINE VectorRegister4Double VectorReciprocalLenEstimate(const VectorRegister4Double& Vector)
{
	return VectorReciprocalLen(Vector);
}

/**
* Loads XYZ and sets W=0
*
* @param Vec	VectorRegister
* @return		VectorRegister(X, Y, Z, 0.0f)
*/
FORCEINLINE VectorRegister4Float VectorSet_W0(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat((Vec).V[0], (Vec).V[1], (Vec).V[2], 0.0f);
}

FORCEINLINE VectorRegister4Double VectorSet_W0(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble((Vec).V[0], (Vec).V[1], (Vec).V[2], 0.0);
}

/**
* Loads XYZ and sets W=1
*
* @param Vec	VectorRegister
* @return		VectorRegister(X, Y, Z, 1.0f)
*/
FORCEINLINE VectorRegister4Float VectorSet_W1(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat((Vec).V[0], (Vec).V[1], (Vec).V[2], 1.0f);
}

FORCEINLINE VectorRegister4Double VectorSet_W1(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble((Vec).V[0], (Vec).V[1], (Vec).V[2], 1.0);
}


// 40% faster version of the Quaternion multiplication.
#define USE_FAST_QUAT_MUL 1

/**
* Multiplies two quaternions; the order matters.
*
* When composing quaternions: VectorQuaternionMultiply(C, A, B) will yield a quaternion C = A * B
* that logically first applies B then A to any subsequent transformation (right first, then left).
*
* @param Result	Pointer to where the result Quat1 * Quat2 should be stored
* @param Quat1	Pointer to the first quaternion (must not be the destination)
* @param Quat2	Pointer to the second quaternion (must not be the destination)
*/
FORCEINLINE void VectorQuaternionMultiply(VectorRegister4Float* RESTRICT Result, const VectorRegister4Float* RESTRICT Quat1, const VectorRegister4Float* RESTRICT Quat2)
{
	typedef float Float4[4];
	const Float4& A = *((const Float4*) Quat1);
	const Float4& B = *((const Float4*) Quat2);
	Float4 & R = *((Float4*) Result);

#if USE_FAST_QUAT_MUL
	const float T0 = (A[2] - A[1]) * (B[1] - B[2]);
	const float T1 = (A[3] + A[0]) * (B[3] + B[0]);
	const float T2 = (A[3] - A[0]) * (B[1] + B[2]);
	const float T3 = (A[1] + A[2]) * (B[3] - B[0]);
	const float T4 = (A[2] - A[0]) * (B[0] - B[1]);
	const float T5 = (A[2] + A[0]) * (B[0] + B[1]);
	const float T6 = (A[3] + A[1]) * (B[3] - B[2]);
	const float T7 = (A[3] - A[1]) * (B[3] + B[2]);
	const float T8 = T5 + T6 + T7;
	const float T9 = 0.5f * (T4 + T8);

	R[0] = T1 + T9 - T8;
	R[1] = T2 + T9 - T7;
	R[2] = T3 + T9 - T6;
	R[3] = T0 + T9 - T5;
#else
	// store intermediate results in temporaries
	const float TX = A[3]*B[0] + A[0]*B[3] + A[1]*B[2] - A[2]*B[1];
	const float TY = A[3]*B[1] - A[0]*B[2] + A[1]*B[3] + A[2]*B[0];
	const float TZ = A[3]*B[2] + A[0]*B[1] - A[1]*B[0] + A[2]*B[3];
	const float TW = A[3]*B[3] - A[0]*B[0] - A[1]*B[1] - A[2]*B[2];

	// copy intermediate result to *this
	R[0] = TX;
	R[1] = TY;
	R[2] = TZ;
	R[3] = TW;
#endif
}

FORCEINLINE void VectorQuaternionMultiply(VectorRegister4Double* RESTRICT Result, const VectorRegister4Double* RESTRICT Quat1, const VectorRegister4Double* RESTRICT Quat2)
{
	typedef double Double4[4];
	const Double4& A = *((const Double4*)Quat1);
	const Double4& B = *((const Double4*)Quat2);
	Double4& R = *((Double4*)Result);

#if USE_FAST_QUAT_MUL
	const double T0 = (A[2] - A[1]) * (B[1] - B[2]);
	const double T1 = (A[3] + A[0]) * (B[3] + B[0]);
	const double T2 = (A[3] - A[0]) * (B[1] + B[2]);
	const double T3 = (A[1] + A[2]) * (B[3] - B[0]);
	const double T4 = (A[2] - A[0]) * (B[0] - B[1]);
	const double T5 = (A[2] + A[0]) * (B[0] + B[1]);
	const double T6 = (A[3] + A[1]) * (B[3] - B[2]);
	const double T7 = (A[3] - A[1]) * (B[3] + B[2]);
	const double T8 = T5 + T6 + T7;
	const double T9 = 0.5 * (T4 + T8);

	R[0] = T1 + T9 - T8;
	R[1] = T2 + T9 - T7;
	R[2] = T3 + T9 - T6;
	R[3] = T0 + T9 - T5;
#else
	// store intermediate results in temporaries
	const double TX = A[3] * B[0] + A[0] * B[3] + A[1] * B[2] - A[2] * B[1];
	const double TY = A[3] * B[1] - A[0] * B[2] + A[1] * B[3] + A[2] * B[0];
	const double TZ = A[3] * B[2] + A[0] * B[1] - A[1] * B[0] + A[2] * B[3];
	const double TW = A[3] * B[3] - A[0] * B[0] - A[1] * B[1] - A[2] * B[2];

	// copy intermediate result to *this
	R[0] = TX;
	R[1] = TY;
	R[2] = TZ;
	R[3] = TW;
#endif
}

/**
* Multiplies two quaternions; the order matters.
*
* Order matters when composing quaternions: C = VectorQuaternionMultiply2(A, B) will yield a quaternion C = A * B
* that logically first applies B then A to any subsequent transformation (right first, then left).
*
* @param Quat1	Pointer to the first quaternion
* @param Quat2	Pointer to the second quaternion
* @return Quat1 * Quat2
*/
FORCEINLINE VectorRegister4Float VectorQuaternionMultiply2( const VectorRegister4Float& Quat1, const VectorRegister4Float& Quat2 )
{
	VectorRegister4Float Result;
	VectorQuaternionMultiply(&Result, &Quat1, &Quat2);
	return Result;
}

FORCEINLINE VectorRegister4Double VectorQuaternionMultiply2(const VectorRegister4Double& Quat1, const VectorRegister4Double& Quat2)
{
	VectorRegister4Double Result;
	VectorQuaternionMultiply(&Result, &Quat1, &Quat2);
	return Result;
}

/**
 * Multiplies two 4x4 matrices.
 *
 * @param Result	Pointer to where the result should be stored
 * @param Matrix1	Pointer to the first matrix
 * @param Matrix2	Pointer to the second matrix
 */
FORCEINLINE void VectorMatrixMultiply(FMatrix44d* Result, const FMatrix44d* Matrix1, const FMatrix44d* Matrix2)
{
	typedef double Double4x4[4][4];
	const Double4x4& A = *((const Double4x4*)Matrix1);
	const Double4x4& B = *((const Double4x4*)Matrix2);
	Double4x4 Temp;
	Temp[0][0] = A[0][0] * B[0][0] + A[0][1] * B[1][0] + A[0][2] * B[2][0] + A[0][3] * B[3][0];
	Temp[0][1] = A[0][0] * B[0][1] + A[0][1] * B[1][1] + A[0][2] * B[2][1] + A[0][3] * B[3][1];
	Temp[0][2] = A[0][0] * B[0][2] + A[0][1] * B[1][2] + A[0][2] * B[2][2] + A[0][3] * B[3][2];
	Temp[0][3] = A[0][0] * B[0][3] + A[0][1] * B[1][3] + A[0][2] * B[2][3] + A[0][3] * B[3][3];

	Temp[1][0] = A[1][0] * B[0][0] + A[1][1] * B[1][0] + A[1][2] * B[2][0] + A[1][3] * B[3][0];
	Temp[1][1] = A[1][0] * B[0][1] + A[1][1] * B[1][1] + A[1][2] * B[2][1] + A[1][3] * B[3][1];
	Temp[1][2] = A[1][0] * B[0][2] + A[1][1] * B[1][2] + A[1][2] * B[2][2] + A[1][3] * B[3][2];
	Temp[1][3] = A[1][0] * B[0][3] + A[1][1] * B[1][3] + A[1][2] * B[2][3] + A[1][3] * B[3][3];

	Temp[2][0] = A[2][0] * B[0][0] + A[2][1] * B[1][0] + A[2][2] * B[2][0] + A[2][3] * B[3][0];
	Temp[2][1] = A[2][0] * B[0][1] + A[2][1] * B[1][1] + A[2][2] * B[2][1] + A[2][3] * B[3][1];
	Temp[2][2] = A[2][0] * B[0][2] + A[2][1] * B[1][2] + A[2][2] * B[2][2] + A[2][3] * B[3][2];
	Temp[2][3] = A[2][0] * B[0][3] + A[2][1] * B[1][3] + A[2][2] * B[2][3] + A[2][3] * B[3][3];

	Temp[3][0] = A[3][0] * B[0][0] + A[3][1] * B[1][0] + A[3][2] * B[2][0] + A[3][3] * B[3][0];
	Temp[3][1] = A[3][0] * B[0][1] + A[3][1] * B[1][1] + A[3][2] * B[2][1] + A[3][3] * B[3][1];
	Temp[3][2] = A[3][0] * B[0][2] + A[3][1] * B[1][2] + A[3][2] * B[2][2] + A[3][3] * B[3][2];
	Temp[3][3] = A[3][0] * B[0][3] + A[3][1] * B[1][3] + A[3][2] * B[2][3] + A[3][3] * B[3][3];
	memcpy(Result, &Temp, 16 * sizeof(double));
}

FORCEINLINE void VectorMatrixMultiply(FMatrix44f* Result, const FMatrix44f* Matrix1, const FMatrix44f* Matrix2)
{
	typedef float Float4x4[4][4];
	const Float4x4& A = *((const Float4x4*)Matrix1);
	const Float4x4& B = *((const Float4x4*)Matrix2);
	Float4x4 Temp;
	Temp[0][0] = A[0][0] * B[0][0] + A[0][1] * B[1][0] + A[0][2] * B[2][0] + A[0][3] * B[3][0];
	Temp[0][1] = A[0][0] * B[0][1] + A[0][1] * B[1][1] + A[0][2] * B[2][1] + A[0][3] * B[3][1];
	Temp[0][2] = A[0][0] * B[0][2] + A[0][1] * B[1][2] + A[0][2] * B[2][2] + A[0][3] * B[3][2];
	Temp[0][3] = A[0][0] * B[0][3] + A[0][1] * B[1][3] + A[0][2] * B[2][3] + A[0][3] * B[3][3];

	Temp[1][0] = A[1][0] * B[0][0] + A[1][1] * B[1][0] + A[1][2] * B[2][0] + A[1][3] * B[3][0];
	Temp[1][1] = A[1][0] * B[0][1] + A[1][1] * B[1][1] + A[1][2] * B[2][1] + A[1][3] * B[3][1];
	Temp[1][2] = A[1][0] * B[0][2] + A[1][1] * B[1][2] + A[1][2] * B[2][2] + A[1][3] * B[3][2];
	Temp[1][3] = A[1][0] * B[0][3] + A[1][1] * B[1][3] + A[1][2] * B[2][3] + A[1][3] * B[3][3];

	Temp[2][0] = A[2][0] * B[0][0] + A[2][1] * B[1][0] + A[2][2] * B[2][0] + A[2][3] * B[3][0];
	Temp[2][1] = A[2][0] * B[0][1] + A[2][1] * B[1][1] + A[2][2] * B[2][1] + A[2][3] * B[3][1];
	Temp[2][2] = A[2][0] * B[0][2] + A[2][1] * B[1][2] + A[2][2] * B[2][2] + A[2][3] * B[3][2];
	Temp[2][3] = A[2][0] * B[0][3] + A[2][1] * B[1][3] + A[2][2] * B[2][3] + A[2][3] * B[3][3];

	Temp[3][0] = A[3][0] * B[0][0] + A[3][1] * B[1][0] + A[3][2] * B[2][0] + A[3][3] * B[3][0];
	Temp[3][1] = A[3][0] * B[0][1] + A[3][1] * B[1][1] + A[3][2] * B[2][1] + A[3][3] * B[3][1];
	Temp[3][2] = A[3][0] * B[0][2] + A[3][1] * B[1][2] + A[3][2] * B[2][2] + A[3][3] * B[3][2];
	Temp[3][3] = A[3][0] * B[0][3] + A[3][1] * B[1][3] + A[3][2] * B[2][3] + A[3][3] * B[3][3];
	memcpy(Result, &Temp, 16 * sizeof(float));
}


/**
 * Calculate the inverse of an FMatrix44.  Src == Dst is allowed
 *
 * @param DstMatrix		FMatrix44 pointer to where the result should be stored
 * @param SrcMatrix		FMatrix44 pointer to the Matrix to be inversed
 * @return bool			returns false if matrix is not invertable and stores identity 
 *
 */
FORCEINLINE bool VectorMatrixInverse(FMatrix44d* DstMatrix, const FMatrix44d* SrcMatrix)
{
	return FMath::MatrixInverse(DstMatrix,SrcMatrix);
}
FORCEINLINE bool VectorMatrixInverse(FMatrix44f* DstMatrix, const FMatrix44f* SrcMatrix)
{
	return FMath::MatrixInverse(DstMatrix,SrcMatrix);
}

/**
 * Calculate Homogeneous transform.
 *
 * @param VecP			VectorRegister
 * @param MatrixM		FMatrix44d pointer to the Matrix to apply transform
 * @return VectorRegister = VecP*MatrixM
 */
FORCEINLINE VectorRegister4Float VectorTransformVector(const VectorRegister4Float& VecP, const FMatrix44f* MatrixM)
{
	typedef float Float4x4[4][4];
	VectorRegister4Float Result;
	const Float4x4& M = *((const Float4x4*)MatrixM);

	Result.V[0] = VecP.V[0] * M[0][0] + VecP.V[1] * M[1][0] + VecP.V[2] * M[2][0] + VecP.V[3] * M[3][0];
	Result.V[1] = VecP.V[0] * M[0][1] + VecP.V[1] * M[1][1] + VecP.V[2] * M[2][1] + VecP.V[3] * M[3][1];
	Result.V[2] = VecP.V[0] * M[0][2] + VecP.V[1] * M[1][2] + VecP.V[2] * M[2][2] + VecP.V[3] * M[3][2];
	Result.V[3] = VecP.V[0] * M[0][3] + VecP.V[1] * M[1][3] + VecP.V[2] * M[2][3] + VecP.V[3] * M[3][3];

	return Result;
}

FORCEINLINE VectorRegister4Float VectorTransformVector(const VectorRegister4Float& VecP, const FMatrix44d* MatrixM)
{
	typedef double Double4x4[4][4];
	VectorRegister4Double Tmp, Result;
	Tmp = MakeVectorRegisterDouble(VecP);
	const Double4x4& M = *((const Double4x4*)MatrixM);

	Result.V[0] = Tmp.V[0] * M[0][0] + Tmp.V[1] * M[1][0] + Tmp.V[2] * M[2][0] + Tmp.V[3] * M[3][0];
	Result.V[1] = Tmp.V[0] * M[0][1] + Tmp.V[1] * M[1][1] + Tmp.V[2] * M[2][1] + Tmp.V[3] * M[3][1];
	Result.V[2] = Tmp.V[0] * M[0][2] + Tmp.V[1] * M[1][2] + Tmp.V[2] * M[2][2] + Tmp.V[3] * M[3][2];
	Result.V[3] = Tmp.V[0] * M[0][3] + Tmp.V[1] * M[1][3] + Tmp.V[2] * M[2][3] + Tmp.V[3] * M[3][3];

	// TODO: this will be a lossy conversion.
	return MakeVectorRegisterFloatFromDouble(Result);
}

FORCEINLINE VectorRegister4Double VectorTransformVector(const VectorRegister4Double& VecP, const FMatrix44d* MatrixM)
{
	typedef double Double4x4[4][4];
	VectorRegister4Double Result;
	const Double4x4& M = *((const Double4x4*)MatrixM);

	Result.V[0] = VecP.V[0] * M[0][0] + VecP.V[1] * M[1][0] + VecP.V[2] * M[2][0] + VecP.V[3] * M[3][0];
	Result.V[1] = VecP.V[0] * M[0][1] + VecP.V[1] * M[1][1] + VecP.V[2] * M[2][1] + VecP.V[3] * M[3][1];
	Result.V[2] = VecP.V[0] * M[0][2] + VecP.V[1] * M[1][2] + VecP.V[2] * M[2][2] + VecP.V[3] * M[3][2];
	Result.V[3] = VecP.V[0] * M[0][3] + VecP.V[1] * M[1][3] + VecP.V[2] * M[2][3] + VecP.V[3] * M[3][3];

	return Result;
}

FORCEINLINE VectorRegister4Double VectorTransformVector(const VectorRegister4Double& VecP, const FMatrix44f* MatrixM)
{
	typedef float Float4x4[4][4];
	typedef double Double4x4[4][4];
	VectorRegister4Double Result;
	const Float4x4& M = *((const Float4x4*)MatrixM);

	Result.V[0] = VecP.V[0] * M[0][0] + VecP.V[1] * M[1][0] + VecP.V[2] * M[2][0] + VecP.V[3] * M[3][0];
	Result.V[1] = VecP.V[0] * M[0][1] + VecP.V[1] * M[1][1] + VecP.V[2] * M[2][1] + VecP.V[3] * M[3][1];
	Result.V[2] = VecP.V[0] * M[0][2] + VecP.V[1] * M[1][2] + VecP.V[2] * M[2][2] + VecP.V[3] * M[3][2];
	Result.V[3] = VecP.V[0] * M[0][3] + VecP.V[1] * M[1][3] + VecP.V[2] * M[2][3] + VecP.V[3] * M[3][3];

	return Result;
}


/**
 * Returns the minimum values of two vectors (component-wise).
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister( min(Vec1.x,Vec2.x), min(Vec1.y,Vec2.y), min(Vec1.z,Vec2.z), min(Vec1.w,Vec2.w) )
 */
FORCEINLINE VectorRegister4Float VectorMin( const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2 )
{
	VectorRegister4Float Vec;
	Vec.V[0] = FMath::Min(Vec1.V[0], Vec2.V[0]);
	Vec.V[1] = FMath::Min(Vec1.V[1], Vec2.V[1]);
	Vec.V[2] = FMath::Min(Vec1.V[2], Vec2.V[2]);
	Vec.V[3] = FMath::Min(Vec1.V[3], Vec2.V[3]);
	return Vec;
}

FORCEINLINE VectorRegister4Double VectorMin(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	VectorRegister4Double Vec;
	Vec.V[0] = FMath::Min(Vec1.V[0], Vec2.V[0]);
	Vec.V[1] = FMath::Min(Vec1.V[1], Vec2.V[1]);
	Vec.V[2] = FMath::Min(Vec1.V[2], Vec2.V[2]);
	Vec.V[3] = FMath::Min(Vec1.V[3], Vec2.V[3]);
	return Vec;
}

/**
 * Returns the maximum values of two vectors (component-wise).
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister( max(Vec1.x,Vec2.x), max(Vec1.y,Vec2.y), max(Vec1.z,Vec2.z), max(Vec1.w,Vec2.w) )
 */
FORCEINLINE VectorRegister4Float VectorMax( const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2 )
{
	VectorRegister4Float Vec;
	Vec.V[0] = FMath::Max(Vec1.V[0], Vec2.V[0]);
	Vec.V[1] = FMath::Max(Vec1.V[1], Vec2.V[1]);
	Vec.V[2] = FMath::Max(Vec1.V[2], Vec2.V[2]);
	Vec.V[3] = FMath::Max(Vec1.V[3], Vec2.V[3]);
	return Vec;
}

FORCEINLINE VectorRegister4Double VectorMax(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	VectorRegister4Double Vec;
	Vec.V[0] = FMath::Max(Vec1.V[0], Vec2.V[0]);
	Vec.V[1] = FMath::Max(Vec1.V[1], Vec2.V[1]);
	Vec.V[2] = FMath::Max(Vec1.V[2], Vec2.V[2]);
	Vec.V[3] = FMath::Max(Vec1.V[3], Vec2.V[3]);
	return Vec;
}

/**
* Creates a vector by combining two high components from each vector
*
* @param Vec1		Source vector1
* @param Vec2		Source vector2
* @return			The combined vector
*/
FORCEINLINE VectorRegister4Float VectorCombineHigh(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	return VectorShuffle(Vec1, Vec2, 2, 3, 2, 3);
}

FORCEINLINE VectorRegister4Double VectorCombineHigh(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	return VectorShuffle(Vec1, Vec2, 2, 3, 2, 3);
}

/**
* Creates a vector by combining two low components from each vector
*
* @param Vec1		Source vector1
* @param Vec2		Source vector2
* @return			The combined vector
*/
FORCEINLINE VectorRegister4Float VectorCombineLow(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	return VectorShuffle(Vec1, Vec2, 0, 1, 0, 1);
}

FORCEINLINE VectorRegister4Double VectorCombineLow(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	return VectorShuffle(Vec1, Vec2, 0, 1, 0, 1);
}


/**
 * Deinterleaves the components of the two given vectors such that the even components
 * are in one vector and the odds in another.
 *
 * @param Lo	[Even0, Odd0, Even1, Odd1]
 * @param Hi	[Even2, Odd2, Even3, Odd3]
 * @param OutEvens [Even0, Even1, Even2, Even3]
 * @param OutOdds [Odd0, Odd1, Odd2, Odd3]
*/
FORCEINLINE void VectorDeinterleave(VectorRegister4Float& RESTRICT OutEvens, VectorRegister4Float& RESTRICT OutOdds, const VectorRegister4Float& RESTRICT Lo, const VectorRegister4Float& RESTRICT Hi)
{
	OutEvens = MakeVectorRegister(Lo.V[0], Lo.V[2], Hi.V[0], Hi.V[2]);
	OutOdds = MakeVectorRegister(Lo.V[1], Lo.V[3], Hi.V[1], Hi.V[3]);
}

FORCEINLINE void VectorDeinterleave(VectorRegister4Double& RESTRICT OutEvens, VectorRegister4Double& RESTRICT OutOdds, const VectorRegister4Double& RESTRICT Lo, const VectorRegister4Double& RESTRICT Hi)
{
	OutEvens = MakeVectorRegister(Lo.V[0], Lo.V[2], Hi.V[0], Hi.V[2]);
	OutOdds = MakeVectorRegister(Lo.V[1], Lo.V[3], Hi.V[1], Hi.V[3]);
}


/**
 * Merges the XYZ components of one vector with the W component of another vector and returns the result.
 *
 * @param VecXYZ	Source vector for XYZ_
 * @param VecW		Source register for ___W (note: the fourth component is used, not the first)
 * @return			VectorRegister(VecXYZ.x, VecXYZ.y, VecXYZ.z, VecW.w)
 */
FORCEINLINE VectorRegister4Float VectorMergeVecXYZ_VecW(const VectorRegister4Float& VecXYZ, const VectorRegister4Float& VecW)
{
	return MakeVectorRegisterFloat(VecXYZ.V[0], VecXYZ.V[1], VecXYZ.V[2], VecW.V[3]);
}

FORCEINLINE VectorRegister4Double VectorMergeVecXYZ_VecW(const VectorRegister4Double& VecXYZ, const VectorRegister4Double& VecW)
{
	return MakeVectorRegisterDouble(VecXYZ.V[0], VecXYZ.V[1], VecXYZ.V[2], VecW.V[3]);
}

/**
 * Loads 4 BYTEs from unaligned memory and converts them into 4 FLOATs.
 * IMPORTANT: You need to call VectorResetFloatRegisters() before using scalar FLOATs after you've used this intrinsic!
 *
 * @param Ptr			Unaligned memory pointer to the 4 BYTEs.
 * @return				VectorRegister( float(Ptr[0]), float(Ptr[1]), float(Ptr[2]), float(Ptr[3]) )
 */
#define VectorLoadByte4( Ptr )			MakeVectorRegisterFloat( float(((const uint8*)(Ptr))[0]), float(((const uint8*)(Ptr))[1]), float(((const uint8*)(Ptr))[2]), float(((const uint8*)(Ptr))[3]) )

 /**
 * Loads 4 signed BYTEs from unaligned memory and converts them into 4 FLOATs.
 * IMPORTANT: You need to call VectorResetFloatRegisters() before using scalar FLOATs after you've used this intrinsic!
 *
 * @param Ptr			Unaligned memory pointer to the 4 BYTEs.
 * @return				VectorRegister( float(Ptr[0]), float(Ptr[1]), float(Ptr[2]), float(Ptr[3]) )
 */
#define VectorLoadSignedByte4( Ptr )	MakeVectorRegisterFloat( float(((const int8*)(Ptr))[0]), float(((const int8*)(Ptr))[1]), float(((const int8*)(Ptr))[2]), float(((const int8*)(Ptr))[3]) )


/**
 * Loads 4 BYTEs from unaligned memory and converts them into 4 FLOATs in reversed order.
 * IMPORTANT: You need to call VectorResetFloatRegisters() before using scalar FLOATs after you've used this intrinsic!
 *
 * @param Ptr			Unaligned memory pointer to the 4 BYTEs.
 * @return				VectorRegister( float(Ptr[3]), float(Ptr[2]), float(Ptr[1]), float(Ptr[0]) )
 */
#define VectorLoadByte4Reverse( Ptr )	MakeVectorRegisterFloat( float(((const uint8*)(Ptr))[3]), float(((const uint8*)(Ptr))[2]), float(((const uint8*)(Ptr))[1]), float(((const uint8*)(Ptr))[0]) )

/**
 * Converts the 4 FLOATs in the vector to 4 BYTEs, clamped to [0,255], and stores to unaligned memory.
 * IMPORTANT: You need to call VectorResetFloatRegisters() before using scalar FLOATs after you've used this intrinsic!
 *
 * @param Vec			Vector containing 4 FLOATs
 * @param Ptr			Unaligned memory pointer to store the 4 BYTEs.
 */
FORCEINLINE void VectorStoreByte4( const VectorRegister4Float& Vec, void* Ptr )
{
	uint8 *BytePtr = (uint8*) Ptr;
	BytePtr[0] = uint8( Vec.V[0] );
	BytePtr[1] = uint8( Vec.V[1] );
	BytePtr[2] = uint8( Vec.V[2] );
	BytePtr[3] = uint8( Vec.V[3] );
}

/**
* Converts the 4 FLOATs in the vector to 4 BYTEs, clamped to [-127,127], and stores to unaligned memory.
* IMPORTANT: You need to call VectorResetFloatRegisters() before using scalar FLOATs after you've used this intrinsic!
*
* @param Vec			Vector containing 4 FLOATs
* @param Ptr			Unaligned memory pointer to store the 4 BYTEs.
*/
FORCEINLINE void VectorStoreSignedByte4(const VectorRegister4Float& Vec, void* Ptr)
{
	int8 *BytePtr = (int8*)Ptr;
	BytePtr[0] = int8(Vec.V[0]);
	BytePtr[1] = int8(Vec.V[1]);
	BytePtr[2] = int8(Vec.V[2]);
	BytePtr[3] = int8(Vec.V[3]);
}


/**
* Loads packed RGB10A2(4 bytes) from unaligned memory and converts them into 4 FLOATs.
* IMPORTANT: You need to call VectorResetFloatRegisters() before using scalar FLOATs after you've used this intrinsic!
*
* @param Ptr			Unaligned memory pointer to the RGB10A2(4 bytes).
* @return				VectorRegister with 4 FLOATs loaded from Ptr.
*/
FORCEINLINE VectorRegister4Float VectorLoadURGB10A2N(void* Ptr)
{
	float V[4];
	uint32 E = *(uint32*)Ptr;

	V[0] = float((E >> 00) & 0x3FF) / 1023.0f;
	V[1] = float((E >> 10) & 0x3FF) / 1023.0f;
	V[2] = float((E >> 20) & 0x3FF) / 1023.0f;
	V[3] = float((E >> 30) & 0x3)   / 3.0f;

	return MakeVectorRegisterFloat(V[0], V[1], V[2], V[3]);
}

/**
* Converts the 4 FLOATs in the vector RGB10A2, clamped to [0, 1023] and [0, 3], and stores to unaligned memory.
* IMPORTANT: You need to call VectorResetFloatRegisters() before using scalar FLOATs after you've used this intrinsic!
*
* @param Vec			Vector containing 4 FLOATs
* @param Ptr			Unaligned memory pointer to store the packed RGB10A2(4 bytes).
*/
FORCEINLINE void VectorStoreURGB10A2N(const VectorRegister4Float& Vec, void* Ptr)
{
	VectorRegister4Float Tmp;
	Tmp = VectorMax(Vec, MakeVectorRegisterFloat(0.0f, 0.0f, 0.0f, 0.0f));
	Tmp = VectorMin(Tmp, MakeVectorRegisterFloat(1.0f, 1.0f, 1.0f, 1.0f));
	Tmp = VectorMultiply(Tmp, MakeVectorRegisterFloat(1023.0f, 1023.0f, 1023.0f, 3.0f));
	
	uint32* Out = (uint32*)Ptr;
	*Out = 
		(uint32(Tmp.V[0]) & 0x3FF) << 00 |
		(uint32(Tmp.V[1]) & 0x3FF) << 10 |
		(uint32(Tmp.V[2]) & 0x3FF) << 20 |
		(uint32(Tmp.V[3]) & 0x003) << 30;
}

/**
 * Returns non-zero if any element in Vec1 is greater than the corresponding element in Vec2, otherwise 0.
 *
 * @param Vec1			1st source vector
 * @param Vec2			2nd source vector
 * @return				Non-zero integer if (Vec1.x > Vec2.x) || (Vec1.y > Vec2.y) || (Vec1.z > Vec2.z) || (Vec1.w > Vec2.w)
 */
FORCEINLINE uint32 VectorAnyGreaterThan(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
{
	// Note: Bitwise OR:ing all results together to avoid branching.
	return (Vec1.V[0] > Vec2.V[0]) | (Vec1.V[1] > Vec2.V[1]) | (Vec1.V[2] > Vec2.V[2]) | (Vec1.V[3] > Vec2.V[3]);
}

FORCEINLINE uint32 VectorAnyGreaterThan(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
{
	// Note: Bitwise OR:ing all results together to avoid branching.
	return (Vec1.V[0] > Vec2.V[0]) | (Vec1.V[1] > Vec2.V[1]) | (Vec1.V[2] > Vec2.V[2]) | (Vec1.V[3] > Vec2.V[3]);
}

/**
 * Resets the floating point registers so that they can be used again.
 * Some intrinsics use these for MMX purposes (e.g. VectorLoadByte4 and VectorStoreByte4).
 */
#define VectorResetFloatRegisters()

/**
 * Returns the control register.
 *
 * @return			The uint32 control register
 */
#define VectorGetControlRegister()		0

/**
 * Sets the control register.
 *
 * @param ControlStatus		The uint32 control status value to set
 */
#define	VectorSetControlRegister(ControlStatus)

/**
 * Control status bit to round all floating point math results towards zero.
 */
#define VECTOR_ROUND_TOWARD_ZERO		0

 /**
  * Denormal operands and results will be flushed to zero
  */
#define VECTOR_DENORMALS_FLUSH_TO_ZERO	0

// Returns true if the vector contains a component that is either NAN or +/-infinite.
inline bool VectorContainsNaNOrInfinite(const VectorRegister4Float& Vec)
{
	return !FMath::IsFinite(Vec.V[0]) || !FMath::IsFinite(Vec.V[1]) || !FMath::IsFinite(Vec.V[2]) || !FMath::IsFinite(Vec.V[3]);
}

inline bool VectorContainsNaNOrInfinite(const VectorRegister4Double& Vec)
{
	return !FMath::IsFinite(Vec.V[0]) || !FMath::IsFinite(Vec.V[1]) || !FMath::IsFinite(Vec.V[2]) || !FMath::IsFinite(Vec.V[3]);
}


FORCEINLINE VectorRegister4Float VectorExp(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::Exp(Vec.V[0]), FMath::Exp(Vec.V[1]), FMath::Exp(Vec.V[2]), FMath::Exp(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorExp(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::Exp(Vec.V[0]), FMath::Exp(Vec.V[1]), FMath::Exp(Vec.V[2]), FMath::Exp(Vec.V[3]));
}


FORCEINLINE VectorRegister4Float VectorExp2(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::Exp2(Vec.V[0]), FMath::Exp2(Vec.V[1]), FMath::Exp2(Vec.V[2]), FMath::Exp2(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorExp2(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::Exp2(Vec.V[0]), FMath::Exp2(Vec.V[1]), FMath::Exp2(Vec.V[2]), FMath::Exp2(Vec.V[3]));
}


FORCEINLINE VectorRegister4Float VectorLog(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::Loge(Vec.V[0]), FMath::Loge(Vec.V[1]), FMath::Loge(Vec.V[2]), FMath::Loge(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorLog(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::Loge(Vec.V[0]), FMath::Loge(Vec.V[1]), FMath::Loge(Vec.V[2]), FMath::Loge(Vec.V[3]));
}


FORCEINLINE VectorRegister4Float VectorLog2(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::Log2(Vec.V[0]), FMath::Log2(Vec.V[1]), FMath::Log2(Vec.V[2]), FMath::Log2(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorLog2(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::Log2(Vec.V[0]), FMath::Log2(Vec.V[1]), FMath::Log2(Vec.V[2]), FMath::Log2(Vec.V[3]));
}


FORCEINLINE VectorRegister4Float VectorSin(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::Sin(Vec.V[0]), FMath::Sin(Vec.V[1]), FMath::Sin(Vec.V[2]), FMath::Sin(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorSin(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::Sin(Vec.V[0]), FMath::Sin(Vec.V[1]), FMath::Sin(Vec.V[2]), FMath::Sin(Vec.V[3]));
}



FORCEINLINE VectorRegister4Float VectorCos(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::Cos(Vec.V[0]), FMath::Cos(Vec.V[1]), FMath::Cos(Vec.V[2]), FMath::Cos(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorCos(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::Cos(Vec.V[0]), FMath::Cos(Vec.V[1]), FMath::Cos(Vec.V[2]), FMath::Cos(Vec.V[3]));
}



FORCEINLINE VectorRegister4Float VectorTan(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::Tan(Vec.V[0]), FMath::Tan(Vec.V[1]), FMath::Tan(Vec.V[2]), FMath::Tan(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorTan(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::Tan(Vec.V[0]), FMath::Tan(Vec.V[1]), FMath::Tan(Vec.V[2]), FMath::Tan(Vec.V[3]));
}


FORCEINLINE VectorRegister4Float VectorASin(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::Asin(Vec.V[0]), FMath::Asin(Vec.V[1]), FMath::Asin(Vec.V[2]), FMath::Asin(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorASin(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::Asin(Vec.V[0]), FMath::Asin(Vec.V[1]), FMath::Asin(Vec.V[2]), FMath::Asin(Vec.V[3]));
}


FORCEINLINE VectorRegister4Float VectorACos(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::Acos(Vec.V[0]), FMath::Acos(Vec.V[1]), FMath::Acos(Vec.V[2]), FMath::Acos(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorACos(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::Acos(Vec.V[0]), FMath::Acos(Vec.V[1]), FMath::Acos(Vec.V[2]), FMath::Acos(Vec.V[3]));
}


FORCEINLINE VectorRegister4Float VectorATan(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::Atan(Vec.V[0]), FMath::Atan(Vec.V[1]), FMath::Atan(Vec.V[2]), FMath::Atan(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorATan(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::Atan(Vec.V[0]), FMath::Atan(Vec.V[1]), FMath::Atan(Vec.V[2]), FMath::Atan(Vec.V[3]));
}


FORCEINLINE VectorRegister4Float VectorATan2(const VectorRegister4Float& Y, const VectorRegister4Float& X)
{
	return MakeVectorRegisterFloat(
		FMath::Atan2(Y.V[0], X.V[0]),
		FMath::Atan2(Y.V[1], X.V[1]),
		FMath::Atan2(Y.V[2], X.V[2]),
		FMath::Atan2(Y.V[3], X.V[3]));
}

FORCEINLINE VectorRegister4Double VectorATan2(const VectorRegister4Double& Y, const VectorRegister4Double& X)
{
	return MakeVectorRegisterDouble(
		FMath::Atan2(Y.V[0], X.V[0]),
		FMath::Atan2(Y.V[1], X.V[1]),
		FMath::Atan2(Y.V[2], X.V[2]),
		FMath::Atan2(Y.V[3], X.V[3]));
}


/**
* Computes the sine and cosine of each component of a Vector.
*
* @param VSinAngles	VectorRegister Pointer to where the Sin result should be stored
* @param VCosAngles	VectorRegister Pointer to where the Cos result should be stored
* @param VAngles VectorRegister Pointer to the input angles
*/
FORCEINLINE void VectorSinCos(VectorRegister4Float* RESTRICT VSinAngles, VectorRegister4Float* RESTRICT VCosAngles, const VectorRegister4Float* RESTRICT VAngles)
{
	union { VectorRegister4Float v; float f[4]; } VecSin, VecCos, VecAngles;
	VecAngles.v = *VAngles;

	FMath::SinCos(&VecSin.f[0], &VecCos.f[0], VecAngles.f[0]);
	FMath::SinCos(&VecSin.f[1], &VecCos.f[1], VecAngles.f[1]);
	FMath::SinCos(&VecSin.f[2], &VecCos.f[2], VecAngles.f[2]);
	FMath::SinCos(&VecSin.f[3], &VecCos.f[3], VecAngles.f[3]);

	*VSinAngles = VecSin.v;
	*VCosAngles = VecCos.v;
}

FORCEINLINE void VectorSinCos(VectorRegister4Double* RESTRICT VSinAngles, VectorRegister4Double* RESTRICT VCosAngles, const VectorRegister4Double* RESTRICT VAngles)
{
	// TODO: No FMath::SinCos<double> function exists yet, but need to revisit if one is added
	*VSinAngles = VectorSin(*VAngles);
	*VCosAngles = VectorCos(*VAngles);
}


FORCEINLINE VectorRegister4Float VectorCeil(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::CeilToFloat(Vec.V[0]), FMath::CeilToFloat(Vec.V[1]), FMath::CeilToFloat(Vec.V[2]), FMath::CeilToFloat(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorCeil(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::CeilToDouble(Vec.V[0]), FMath::CeilToDouble(Vec.V[1]), FMath::CeilToDouble(Vec.V[2]), FMath::CeilToDouble(Vec.V[3]));
}


FORCEINLINE VectorRegister4Float VectorFloor(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::FloorToFloat(Vec.V[0]), FMath::FloorToFloat(Vec.V[1]), FMath::FloorToFloat(Vec.V[2]), FMath::FloorToFloat(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorFloor(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::FloorToDouble(Vec.V[0]), FMath::FloorToDouble(Vec.V[1]), FMath::FloorToDouble(Vec.V[2]), FMath::FloorToDouble(Vec.V[3]));
}


FORCEINLINE VectorRegister4Float VectorTruncate(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::TruncToFloat(Vec.V[0]), FMath::TruncToFloat(Vec.V[1]), FMath::TruncToFloat(Vec.V[2]), FMath::TruncToFloat(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorTruncate(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::TruncToDouble(Vec.V[0]), FMath::TruncToDouble(Vec.V[1]), FMath::TruncToDouble(Vec.V[2]), FMath::TruncToDouble(Vec.V[3]));
}


FORCEINLINE VectorRegister4Float VectorRound(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(FMath::RoundToFloat(Vec.V[0]), FMath::RoundToFloat(Vec.V[1]), FMath::RoundToFloat(Vec.V[2]), FMath::RoundToFloat(Vec.V[3]));
}

FORCEINLINE VectorRegister4Double VectorRound(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(FMath::RoundToDouble(Vec.V[0]), FMath::RoundToDouble(Vec.V[1]), FMath::RoundToDouble(Vec.V[2]), FMath::RoundToDouble(Vec.V[3]));
}


FORCEINLINE VectorRegister4Int VectorRoundToIntHalfToEven(const VectorRegister4Float& A)
{
	return MakeVectorRegisterInt(
		(int32)FMath::RoundHalfToEven(A.V[0]),
		(int32)FMath::RoundHalfToEven(A.V[1]),
		(int32)FMath::RoundHalfToEven(A.V[2]),
		(int32)FMath::RoundHalfToEven(A.V[3]));
}


FORCEINLINE VectorRegister4Float VectorMod(const VectorRegister4Float& X, const VectorRegister4Float& Y)
{
	return MakeVectorRegisterFloat(
		FMath::Fmod(X.V[0], Y.V[0]),
		FMath::Fmod(X.V[1], Y.V[1]),
		FMath::Fmod(X.V[2], Y.V[2]),
		FMath::Fmod(X.V[3], Y.V[3]));
}

FORCEINLINE VectorRegister4Double VectorMod(const VectorRegister4Double& X, const VectorRegister4Double& Y)
{
	return MakeVectorRegisterDouble(
		FMath::Fmod(X.V[0], Y.V[0]),
		FMath::Fmod(X.V[1], Y.V[1]),
		FMath::Fmod(X.V[2], Y.V[2]),
		FMath::Fmod(X.V[3], Y.V[3]));
}


FORCEINLINE VectorRegister4Float VectorSign(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(
		(float)(Vec.V[0] >= 0.0f ? 1.0f : -1.0f),
		(float)(Vec.V[1] >= 0.0f ? 1.0f : -1.0f),
		(float)(Vec.V[2] >= 0.0f ? 1.0f : -1.0f),
		(float)(Vec.V[3] >= 0.0f ? 1.0f : -1.0f));
}

FORCEINLINE VectorRegister4Double VectorSign(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(
		(double)(Vec.V[0] >= 0.0 ? 1.0 : -1.0),
		(double)(Vec.V[1] >= 0.0 ? 1.0 : -1.0),
		(double)(Vec.V[2] >= 0.0 ? 1.0 : -1.0),
		(double)(Vec.V[3] >= 0.0 ? 1.0 : -1.0));
}


FORCEINLINE VectorRegister4Float VectorStep(const VectorRegister4Float& Vec)
{
	return MakeVectorRegisterFloat(
		(float)(Vec.V[0] >= 0.0f ? 1.0f : 0.0f),
		(float)(Vec.V[1] >= 0.0f ? 1.0f : 0.0f),
		(float)(Vec.V[2] >= 0.0f ? 1.0f : 0.0f),
		(float)(Vec.V[3] >= 0.0f ? 1.0f : 0.0f));
}

FORCEINLINE VectorRegister4Double VectorStep(const VectorRegister4Double& Vec)
{
	return MakeVectorRegisterDouble(
		(double)(Vec.V[0] >= 0.0 ? 1.0 : 0.0),
		(double)(Vec.V[1] >= 0.0 ? 1.0 : 0.0),
		(double)(Vec.V[2] >= 0.0 ? 1.0 : 0.0),
		(double)(Vec.V[3] >= 0.0 ? 1.0 : 0.0));
}

/**
* Loads packed RGBA16(4 bytes) from unaligned memory and converts them into 4 FLOATs.
* IMPORTANT: You need to call VectorResetFloatRegisters() before using scalar FLOATs after you've used this intrinsic!
*
* @param Ptr			Unaligned memory pointer to the RGBA16(8 bytes).
* @return				VectorRegister with 4 FLOATs loaded from Ptr.
*/
FORCEINLINE VectorRegister4Float VectorLoadURGBA16N(void* Ptr)
{
	float V[4];
	uint16* E = (uint16*)Ptr;

	V[0] = float(E[0]);
	V[1] = float(E[1]);
	V[2] = float(E[2]);
	V[3] = float(E[3]);

	return MakeVectorRegisterFloat(V[0], V[1], V[2], V[3]);
}

/**
* Loads packed signed RGBA16(4 bytes) from unaligned memory and converts them into 4 FLOATs.
* IMPORTANT: You need to call VectorResetFloatRegisters() before using scalar FLOATs after you've used this intrinsic!
*
* @param Ptr			Unaligned memory pointer to the RGBA16(8 bytes).
* @return				VectorRegister with 4 FLOATs loaded from Ptr.
*/
FORCEINLINE VectorRegister4Float VectorLoadSRGBA16N(void* Ptr)
{
	float V[4];
	int16* E = (int16*)Ptr;

	V[0] = float(E[0]);
	V[1] = float(E[1]);
	V[2] = float(E[2]);
	V[3] = float(E[3]);

	return MakeVectorRegisterFloat(V[0], V[1], V[2], V[3]);
}

/**
* Converts the 4 FLOATs in the vector RGBA16, clamped to [0, 65535], and stores to unaligned memory.
* IMPORTANT: You need to call VectorResetFloatRegisters() before using scalar FLOATs after you've used this intrinsic!
*
* @param Vec			Vector containing 4 FLOATs
* @param Ptr			Unaligned memory pointer to store the packed RGBA16(8 bytes).
*/
FORCEINLINE void VectorStoreURGBA16N(const VectorRegister4Float& Vec, void* Ptr)
{
	VectorRegister4Float Tmp;
	Tmp = VectorMax(Vec, MakeVectorRegisterFloat(0.0f, 0.0f, 0.0f, 0.0f));
	Tmp = VectorMin(Tmp, MakeVectorRegisterFloat(1.0f, 1.0f, 1.0f, 1.0f));
	Tmp = VectorMultiplyAdd(Tmp, MakeVectorRegisterFloat(65535.0f, 65535.0f, 65535.0f, 65535.0f), MakeVectorRegisterFloat(0.5f, 0.5f, 0.5f, 0.5f));
	Tmp = VectorTruncate(Tmp);

	uint16* Out = (uint16*)Ptr;
	Out[0] = (uint16)Tmp.V[0];
	Out[1] = (uint16)Tmp.V[0];
	Out[2] = (uint16)Tmp.V[0];
	Out[3] = (uint16)Tmp.V[0];
}

//////////////////////////////////////////////////////////////////////////
//Integer ops

//Bitwise
/** = a & b */
FORCEINLINE VectorRegister4Int VectorIntAnd(const VectorRegister4Int& A, const VectorRegister4Int& B)
{
	return MakeVectorRegisterInt(
		A.V[0] & B.V[0],
		A.V[1] & B.V[1],
		A.V[2] & B.V[2],
		A.V[3] & B.V[3]);
}

/** = a | b */
FORCEINLINE VectorRegister4Int VectorIntOr(const VectorRegister4Int& A, const VectorRegister4Int& B)
{
	return MakeVectorRegisterInt(
		A.V[0] | B.V[0],
		A.V[1] | B.V[1],
		A.V[2] | B.V[2],
		A.V[3] | B.V[3]);
}
/** = a ^ b */
FORCEINLINE VectorRegister4Int VectorIntXor(const VectorRegister4Int& A, const VectorRegister4Int& B)
{
	return MakeVectorRegisterInt(
		A.V[0] ^ B.V[0],
		A.V[1] ^ B.V[1],
		A.V[2] ^ B.V[2],
		A.V[3] ^ B.V[3]);
}

/** = (~a) & b to match _mm_andnot_si128 */
FORCEINLINE VectorRegister4Int VectorIntAndNot(const VectorRegister4Int& A, const VectorRegister4Int& B)
{
	return MakeVectorRegisterInt(
		(~A.V[0]) & B.V[0],
		(~A.V[1]) & B.V[1],
		(~A.V[2]) & B.V[2],
		(~A.V[3]) & B.V[3]);
}
/** = ~a */
FORCEINLINE VectorRegister4Int VectorIntNot(const VectorRegister4Int& A)
{
	return MakeVectorRegisterInt(
		~A.V[0],
		~A.V[1],
		~A.V[2],
		~A.V[3]);
}

//Comparison
FORCEINLINE VectorRegister4Int VectorIntCompareEQ(const VectorRegister4Int& A, const VectorRegister4Int& B)
{
	return MakeVectorRegisterInt(
		A.V[0] == B.V[0] ? 0xFFFFFFFF : 0,
		A.V[1] == B.V[1] ? 0xFFFFFFFF : 0,
		A.V[2] == B.V[2] ? 0xFFFFFFFF : 0,
		A.V[3] == B.V[3] ? 0xFFFFFFFF : 0);
}

FORCEINLINE VectorRegister4Int VectorIntCompareNEQ(const VectorRegister4Int& A, const VectorRegister4Int& B)
{
	return MakeVectorRegisterInt(
		A.V[0] != B.V[0] ? 0xFFFFFFFF : 0,
		A.V[1] != B.V[1] ? 0xFFFFFFFF : 0,
		A.V[2] != B.V[2] ? 0xFFFFFFFF : 0,
		A.V[3] != B.V[3] ? 0xFFFFFFFF : 0);
}

FORCEINLINE VectorRegister4Int VectorIntCompareGT(const VectorRegister4Int& A, const VectorRegister4Int& B)
{
	return MakeVectorRegisterInt(
		A.V[0] > B.V[0] ? 0xFFFFFFFF : 0,
		A.V[1] > B.V[1] ? 0xFFFFFFFF : 0,
		A.V[2] > B.V[2] ? 0xFFFFFFFF : 0,
		A.V[3] > B.V[3] ? 0xFFFFFFFF : 0);
}

FORCEINLINE VectorRegister4Int VectorIntCompareLT(const VectorRegister4Int& A, const VectorRegister4Int& B)
{
	return MakeVectorRegisterInt(
		A.V[0] < B.V[0] ? 0xFFFFFFFF : 0,
		A.V[1] < B.V[1] ? 0xFFFFFFFF : 0,
		A.V[2] < B.V[2] ? 0xFFFFFFFF : 0,
		A.V[3] < B.V[3] ? 0xFFFFFFFF : 0);
}

FORCEINLINE VectorRegister4Int VectorIntCompareGE(const VectorRegister4Int& A, const VectorRegister4Int& B)
{
	return MakeVectorRegisterInt(
		A.V[0] >= B.V[0] ? 0xFFFFFFFF : 0,
		A.V[1] >= B.V[1] ? 0xFFFFFFFF : 0,
		A.V[2] >= B.V[2] ? 0xFFFFFFFF : 0,
		A.V[3] >= B.V[3] ? 0xFFFFFFFF : 0);
}

FORCEINLINE VectorRegister4Int VectorIntCompareLE(const VectorRegister4Int& A, const VectorRegister4Int& B)
{
	return MakeVectorRegisterInt(
		A.V[0] <= B.V[0] ? 0xFFFFFFFF : 0,
		A.V[1] <= B.V[1] ? 0xFFFFFFFF : 0,
		A.V[2] <= B.V[2] ? 0xFFFFFFFF : 0,
		A.V[3] <= B.V[3] ? 0xFFFFFFFF : 0);
}


FORCEINLINE VectorRegister4Int VectorIntSelect(const VectorRegister4Int& Mask, const VectorRegister4Int& Vec1, const VectorRegister4Int& Vec2)
{
	return VectorIntXor(Vec2, VectorIntAnd(Mask, VectorIntXor(Vec1, Vec2)));
}

//Arithmetic
FORCEINLINE VectorRegister4Int VectorIntAdd(const VectorRegister4Int& A, const VectorRegister4Int& B)
{
	return MakeVectorRegisterInt(
		A.V[0] + B.V[0],
		A.V[1] + B.V[1],
		A.V[2] + B.V[2],
		A.V[3] + B.V[3]);
}

FORCEINLINE VectorRegister4Int VectorIntSubtract(const VectorRegister4Int& A, const VectorRegister4Int& B)
{
	return MakeVectorRegisterInt(
		A.V[0] - B.V[0],
		A.V[1] - B.V[1],
		A.V[2] - B.V[2],
		A.V[3] - B.V[3]);
}

FORCEINLINE VectorRegister4Int VectorIntMultiply(const VectorRegister4Int& A, const VectorRegister4Int& B)
{
	return MakeVectorRegisterInt(
		A.V[0] * B.V[0],
		A.V[1] * B.V[1],
		A.V[2] * B.V[2],
		A.V[3] * B.V[3]);
}

FORCEINLINE VectorRegister4Int VectorIntNegate(const VectorRegister4Int& A)
{
	return MakeVectorRegisterInt(
		-A.V[0],
		-A.V[1],
		-A.V[2],
		-A.V[3]);
}

FORCEINLINE VectorRegister4Int VectorIntMin(const VectorRegister4Int& A, const VectorRegister4Int& B)
{
	return MakeVectorRegisterInt(
		FMath::Min(A.V[0] , B.V[0]),
		FMath::Min(A.V[1] , B.V[1]),
		FMath::Min(A.V[2] , B.V[2]),
		FMath::Min(A.V[3] , B.V[3]));
}

FORCEINLINE VectorRegister4Int VectorIntMax(const VectorRegister4Int& A, const VectorRegister4Int& B)
{
	return MakeVectorRegisterInt(
		FMath::Max(A.V[0], B.V[0]),
		FMath::Max(A.V[1], B.V[1]),
		FMath::Max(A.V[2], B.V[2]),
		FMath::Max(A.V[3], B.V[3]));
}

FORCEINLINE VectorRegister4Int VectorIntAbs(const VectorRegister4Int& A)
{
	return MakeVectorRegisterInt(
		FMath::Abs(A.V[0]),
		FMath::Abs(A.V[1]),
		FMath::Abs(A.V[2]),
		FMath::Abs(A.V[3]));
}

#define VectorIntSign(A) VectorIntSelect( VectorIntCompareGE(A, GlobalVectorConstants::IntZero), GlobalVectorConstants::IntOne, GlobalVectorConstants::IntMinusOne )

FORCEINLINE VectorRegister4Float VectorIntToFloat(const VectorRegister4Int& A)
{
	return MakeVectorRegisterFloat(
		(float)A.V[0],
		(float)A.V[1],
		(float)A.V[2],
		(float)A.V[3]);
}

FORCEINLINE VectorRegister4Int VectorFloatToInt(const VectorRegister4Float& A)
{
	return MakeVectorRegisterInt(
		(int32)A.V[0],
		(int32)A.V[1],
		(int32)A.V[2],
		(int32)A.V[3]);
}

// TODO: LWC: potential loss of data
FORCEINLINE VectorRegister4Int VectorFloatToInt(const VectorRegister4Double& A)
{
	return MakeVectorRegisterInt(
		(int32)A.V[0],
		(int32)A.V[1],
		(int32)A.V[2],
		(int32)A.V[3]);
}

FORCEINLINE VectorRegister4Int VectorDoubleToInt(const VectorRegister4Double& A)
{
	return VectorFloatToInt(A);
}

FORCEINLINE VectorRegister4Int VectorShuffleByte4(const VectorRegister4Int& Vec, const VectorRegister4Int& Mask)
{
	const int8* VecPtr = (const int8*)&Vec;
	const uint8* MaskPtr = (const uint8*)&Mask;
	
	VectorRegister4Int OutVec;
	int8* OutPtr = (int8*)&OutVec;
	
	constexpr uint32 NumBytes = sizeof(VectorRegister4Int);

	// We use 0x10 here to denote zero fill which is compatible with both SSE, which uses >= 0x80,
	// and Neon, which uses >= number of source bytes
	constexpr uint32 ZeroFill = NumBytes;

	for (uint32 Index = 0; Index < NumBytes; ++Index)
	{
		uint32 M = MaskPtr[Index];
		OutPtr[Index] = (M < ZeroFill) ? VecPtr[M] : 0;
	}
	
	return OutVec;
}

//Loads and stores

/**
* Stores a vector to memory (aligned or unaligned).
*
* @param Vec	Vector to store
* @param Ptr	Memory pointer
*/
FORCEINLINE void VectorIntStore(const VectorRegister4Int& A, const void* Ptr)
{
	int32* IntPtr = (int32*)Ptr;	
	IntPtr[0] = A.V[0];
	IntPtr[1] = A.V[1];
	IntPtr[2] = A.V[2];
	IntPtr[3] = A.V[3];
}

/**
* Loads 4 int32s from unaligned memory.
*
* @param Ptr	Unaligned memory pointer to the 4 int32s
* @return		VectorRegister4Int(Ptr[0], Ptr[1], Ptr[2], Ptr[3])
*/

FORCEINLINE VectorRegister4Int VectorIntLoad(const void* Ptr)
{
	int32* IntPtr = (int32*)Ptr;
	return MakeVectorRegisterInt(
		IntPtr[0],
		IntPtr[1],
		IntPtr[2],
		IntPtr[3]);
}

/**
* Stores a vector to memory (aligned).
*
* @param Vec	Vector to store
* @param Ptr	Aligned Memory pointer
*/
FORCEINLINE void VectorIntStoreAligned(const VectorRegister4Int& A, const void* Ptr)
{
	int32* IntPtr = (int32*)Ptr;
	IntPtr[0] = A.V[0];
	IntPtr[1] = A.V[1];
	IntPtr[2] = A.V[2];
	IntPtr[3] = A.V[3];
}

/**
* Loads 4 int32s from aligned memory.
*
* @param Ptr	Aligned memory pointer to the 4 int32s
* @return		VectorRegister4Int(Ptr[0], Ptr[1], Ptr[2], Ptr[3])
*/
FORCEINLINE VectorRegister4Int VectorIntLoadAligned(const void* Ptr)
{
	int32* IntPtr = (int32*)Ptr;
	return MakeVectorRegisterInt(
		IntPtr[0],
		IntPtr[1],
		IntPtr[2],
		IntPtr[3]);
}

/**
* Loads 1 int32 from unaligned memory into all components of a vector register.
*
* @param Ptr	Unaligned memory pointer to the 4 int32s
* @return		VectorRegister4Int(*Ptr, *Ptr, *Ptr, *Ptr)
*/
FORCEINLINE VectorRegister4Int VectorIntLoad1(const void* Ptr)
{
	int32 IntSplat = *(int32*)Ptr;

	return MakeVectorRegisterInt(
		IntSplat,
		IntSplat,
		IntSplat,
		IntSplat);
}

/**
* Loads 2 int64s from unaligned memory.
*
* @param Ptr	Unaligned memory pointer to the 2 int64s
* @return		VectorRegister4Int(Ptr[0], Ptr[1])
*/

FORCEINLINE VectorRegister4Int VectorInt64Load(const void* Ptr)
{
	int64* IntPtr = (int64*)Ptr;
	return MakeVectorRegisterInt64(
		IntPtr[0],
		IntPtr[1]);
}

/**
* Loads 2 doubles from unaligned memory.
*
* @param Ptr	Unaligned memory pointer to the 2 doubles
* @return		VectorRegister2Double(Ptr[0], Ptr[1])
*/

FORCEINLINE VectorRegister2Double VectorDoubleLoad(const void* Ptr)
{
	double* DoublePtr = (double*)Ptr;
	return MakeVectorRegister2Double(
		DoublePtr[0],
		DoublePtr[1]);
}

#define VectorSetZero()								MakeVectorRegisterFloat(0.f, 0.f, 0.f, 0.f)
#define VectorSet1(F)								VectorSetFloat1(F)
#define VectorIntSet1(F)							MakeVectorRegisterInt(F, F, F, F)
#define VectorCastIntToFloat(Vec)                   VectorLoad((float*)(Vec.V))
#define VectorCastFloatToInt(Vec)                   VectorIntLoad(Vec.V)
#define VectorCastDoubleToInt(Vec)                  VectorInt64Load(Vec.V)
#define VectorCastIntToDouble(Vec)                  VectorDoubleLoad(Vec.V)
#define VectorShuffleImmediate(Vec, I0, I1, I2, I3) VectorShuffle(Vec, Vec, I3, I2, I1, I0) // ShuffleImmediate shuffle is reversed from our logical shuffle.
#define VectorShiftLeftImm(Vec, ImmAmt)             static_assert(false, "Unimplemented") // TODO: implement
#define VectorShiftRightImmArithmetic(Vec, ImmAmt)  static_assert(false, "Unimplemented") // TODO: implement
#define VectorShiftRightImmLogical(Vec, ImmAmt)		static_assert(false, "Unimplemented") // TODO: implement
#define VectorIntExpandLow16To32(V0)				static_assert(false, "Unimplemented") // TODO: implement
