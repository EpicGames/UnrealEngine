// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

// HEADER_UNIT_SKIP - Not included directly

PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS

#if PLATFORM_ENABLE_VECTORINTRINSICS_NEON

struct VectorRegisterConstInit {};

// Include the intrinsic functions header
#if (PLATFORM_WINDOWS && !PLATFORM_COMPILER_CLANG)
#include <arm64_neon.h>
#else
#include <arm_neon.h>
#endif

#include "Math/Float16.h"

/*=============================================================================
 *	Helpers:
 *============================================================================*/

#if PLATFORM_COMPILER_MSVC

// MSVC NEON headers typedef float32x4_t and int32x4_t both to __n128
// This wrapper type allows VectorRegister4Float and VectorRegister4Int to be
// discriminated for template specialization (e.g. FConstantHandler)
//
// This comes at the cost of having to define constructors for some
// anonymous unions, because VectorRegister4Float/VectorRegister4Int are no
// longer trivially constructible. The optimizer should eliminate the
// redundant zero initialization in these cases for non-MSVC (e.g. V()
// is called now where it wasn't before)
template<typename T, typename BASE_TYPE>
struct alignas(alignof(T)) VectorRegisterWrapper
{
	FORCEINLINE VectorRegisterWrapper() = default;
	FORCEINLINE constexpr VectorRegisterWrapper(T vec) : m_vec(vec) {}

	FORCEINLINE operator T&() { return m_vec; }
	FORCEINLINE operator const T&() const { return m_vec; }

	FORCEINLINE BASE_TYPE operator[](int Index) const;

	T m_vec;
};

template<>
FORCEINLINE float VectorRegisterWrapper<float32x4_t, float>::operator[](int Index) const
{
	return m_vec.n128_f32[Index];
}

template<>
FORCEINLINE double VectorRegisterWrapper<float64x2_t, double>::operator[](int Index) const
{
	return m_vec.n128_f64[Index];
}

template<>
FORCEINLINE int VectorRegisterWrapper<int32x4_t, int>::operator[](int Index) const
{
	return m_vec.n128_i32[Index];
}

template<>
FORCEINLINE int64 VectorRegisterWrapper<int64x2_t, int64>::operator[](int Index) const
{
	return m_vec.n128_i64[Index];
}

/** 16-byte vector register type */
typedef VectorRegisterWrapper<float32x4_t, float> VectorRegister4Float;
typedef VectorRegisterWrapper<float64x2_t, double> VectorRegister2Double;
typedef VectorRegisterWrapper<int32x4_t, int> VectorRegister4Int;
typedef VectorRegisterWrapper<int64x2_t, int64> VectorRegister2Int64;

typedef float32x4x4_t VectorRegister4x4Float;

FORCEINLINE constexpr VectorRegister4Int MakeVectorRegisterIntConstant(int32 X, int32 Y, int32 Z, int32 W)
{
    int32x4_t Out = {};
	Out.n128_i32[0] = X;
    Out.n128_i32[1] = Y;
    Out.n128_i32[2] = Z;
    Out.n128_i32[3] = W;
    return Out;
}

FORCEINLINE constexpr VectorRegister4Float MakeVectorRegisterFloatConstant(float X, float Y, float Z, float W)
{
    float32x4_t Out = {};
    Out.n128_f32[0] = X;
    Out.n128_f32[1] = Y;
    Out.n128_f32[2] = Z;
    Out.n128_f32[3] = W;
	return Out;
}

FORCEINLINE constexpr VectorRegister2Double MakeVectorRegister2DoubleConstant(double X, double Y)
{
    float64x2_t Out = {};
    Out.n128_f64[0] = X;
    Out.n128_f64[1] = Y;
	return Out;
}

#else

/** 16-byte vector register type */
typedef float32x4_t GCC_ALIGN(16) VectorRegister4Float;
typedef float64x2_t GCC_ALIGN(16) VectorRegister2Double;
typedef int32x4_t  GCC_ALIGN(16) VectorRegister4Int;
typedef int64x2_t GCC_ALIGN(16) VectorRegister2Int64;
typedef float32x4x4_t GCC_ALIGN(16) VectorRegister4x4Float;

FORCEINLINE constexpr VectorRegister4Int MakeVectorRegisterIntConstant(int32 X, int32 Y, int32 Z, int32 W)
{
    return VectorRegister4Int { X, Y, Z, W };
}

FORCEINLINE constexpr VectorRegister4Float MakeVectorRegisterFloatConstant(float X, float Y, float Z, float W)
{
    return VectorRegister4Float { X, Y, Z, W };
}

FORCEINLINE constexpr VectorRegister2Double MakeVectorRegister2DoubleConstant(double X, double Y)
{
    return VectorRegister2Double { X, Y };
}

#endif

#define DECLARE_VECTOR_REGISTER(X, Y, Z, W) MakeVectorRegister( X, Y, Z, W )

struct alignas(16) VectorRegister4Double
{
	struct
	{
		VectorRegister2Double XY;
		VectorRegister2Double ZW;
	};

	FORCEINLINE VectorRegister4Double() = default;
	FORCEINLINE VectorRegister4Double(VectorRegister2Double xy, VectorRegister2Double zw) : XY(xy), ZW(zw) {}
	FORCEINLINE constexpr VectorRegister4Double(VectorRegister2Double xy, VectorRegister2Double zw, VectorRegisterConstInit) : XY(xy), ZW(zw) {}

	FORCEINLINE VectorRegister4Double(VectorRegister4Float From)
	{
		XY = vcvt_f64_f32(*(float32x2_t*)&From);
		ZW = vcvt_high_f64_f32(From);
	}

	VectorRegister4Double(VectorRegister2Double From) = delete;

	FORCEINLINE VectorRegister4Double& operator=(VectorRegister4Float From)
	{
		*this = VectorRegister4Double(From);
		return *this;
	}
};

typedef VectorRegister4Double VectorRegister;
#define VectorZeroVectorRegister() VectorZeroDouble()
#define VectorOneVectorRegister() VectorOneDouble()

// Forward declarations
VectorRegister4Float VectorLoadAligned(const float* Ptr);
VectorRegister4Double VectorLoadAligned(const double* Ptr);
void VectorStoreAligned(VectorRegister4Float Vec, float* Ptr);
void VectorStoreAligned(VectorRegister4Double Vec, double* Dst);


// Helper for conveniently aligning a float array for extraction from VectorRegister4Float
struct alignas(alignof(VectorRegister4Float)) AlignedFloat4
{
	float V[4];

	FORCEINLINE AlignedFloat4(VectorRegister4Float Vec)
	{
		VectorStoreAligned(Vec, V);
	}

	FORCEINLINE float operator[](int32 Index) const { return V[Index]; }
	FORCEINLINE float& operator[](int32 Index) { return V[Index]; }

	FORCEINLINE VectorRegister4Float ToVectorRegister() const
	{
		return VectorLoadAligned(V);
	}
};


// Helper for conveniently aligning a double array for extraction from VectorRegister4Double
struct alignas(alignof(VectorRegister4Double)) AlignedDouble4
{
	double V[4];

	FORCEINLINE AlignedDouble4(VectorRegister4Double Vec)
	{
		VectorStoreAligned(Vec, V);
	}

	FORCEINLINE double operator[](int32 Index) const { return V[Index]; }
	FORCEINLINE double& operator[](int32 Index) { return V[Index]; }

	FORCEINLINE VectorRegister4Double ToVectorRegister() const
	{
		return VectorLoadAligned(V);
	}
};

typedef AlignedDouble4 AlignedRegister4;

// Aliases
typedef VectorRegister4Int VectorRegister4i;
typedef VectorRegister4Float VectorRegister4f;
typedef VectorRegister4Double VectorRegister4d;
typedef VectorRegister2Double VectorRegister2d;

// Backwards compatibility
typedef VectorRegister4Double VectorRegister4;
typedef VectorRegister4 VectorRegister;
typedef VectorRegister4Int VectorRegisterInt;

/**
 * Returns a bitwise equivalent vector based on 4 uint32s.
 *
 * @param X		1st uint32 component
 * @param Y		2nd uint32 component
 * @param Z		3rd uint32 component
 * @param W		4th uint32 component
 * @return		Bitwise equivalent vector with 4 floats
 */
FORCEINLINE VectorRegister4Float MakeVectorRegister(uint32 X, uint32 Y, uint32 Z, uint32 W)
{
	union U {
		VectorRegister4Float V; uint32 F[4];
		FORCEINLINE U() : V() {}
	} Tmp;
	Tmp.F[0] = X;
	Tmp.F[1] = Y;
	Tmp.F[2] = Z;
	Tmp.F[3] = W;
	return Tmp.V;
}

FORCEINLINE VectorRegister4Float MakeVectorRegisterFloat(uint32 X, uint32 Y, uint32 Z, uint32 W)
{
	return MakeVectorRegister(X, Y, Z, W);
}

// Nicer alias
FORCEINLINE VectorRegister4Float MakeVectorRegisterFloatMask(uint32 X, uint32 Y, uint32 Z, uint32 W)
{
	return MakeVectorRegisterFloat(X, Y, Z, W);
}


/**
 * Returns a vector based on 4 floats.
 *
 * @param X		1st float component
 * @param Y		2nd float component
 * @param Z		3rd float component
 * @param W		4th float component
 * @return		Vector of the 4 floats
 */
FORCEINLINE VectorRegister4Float MakeVectorRegister(float X, float Y, float Z, float W)
{
	union U {
		VectorRegister4Float V; float F[4];
		FORCEINLINE U() : V() {}
	} Tmp;
	Tmp.F[0] = X;
	Tmp.F[1] = Y;
	Tmp.F[2] = Z;
	Tmp.F[3] = W;
	return Tmp.V;
}

FORCEINLINE VectorRegister4Float MakeVectorRegisterFloat(float X, float Y, float Z, float W)
{
	return MakeVectorRegister(X, Y, Z, W);
}

/**
 * Returns a vector based on 4 doubles.
 *
 * @param X		1st double component
 * @param Y		2nd double component
 * @param Z		3rd double component
 * @param W		4th double component
 * @return		Vector of the 4 doubles
 */
FORCEINLINE VectorRegister4Double MakeVectorRegister(double X, double Y, double Z, double W)
{
	union U
	{
		VectorRegister4Double V; double D[4];
		FORCEINLINE U() : V() {}
	} Tmp;
	Tmp.D[0] = X;
	Tmp.D[1] = Y;
	Tmp.D[2] = Z;
	Tmp.D[3] = W;
	return Tmp.V;
}

FORCEINLINE VectorRegister4Double MakeVectorRegisterDouble(double X, double Y, double Z, double W)
{
	return MakeVectorRegister(X, Y, Z, W);
}

FORCEINLINE VectorRegister4Double MakeVectorRegisterDouble(VectorRegister2Double XY, VectorRegister2Double ZW)
{
	return VectorRegister4Double(XY, ZW);
}

FORCEINLINE VectorRegister4Double MakeVectorRegisterDouble(uint64 X, uint64 Y, uint64 Z, uint64 W)
{
	union U
	{
		VectorRegister4Double V; uint64_t D[4];
		FORCEINLINE U() : V() {}
	} Tmp;
	Tmp.D[0] = X;
	Tmp.D[1] = Y;
	Tmp.D[2] = Z;
	Tmp.D[3] = W;
	return Tmp.V;
}

// Nicer alias
FORCEINLINE VectorRegister4Double MakeVectorRegisterDoubleMask(uint64 X, uint64 Y, uint64 Z, uint64 W)
{
	return MakeVectorRegisterDouble(X, Y, Z, W);
}

FORCEINLINE VectorRegister2Double MakeVectorRegister2Double(double X, double Y)
{
	union U
	{
		VectorRegister2Double V; double D[2];
		FORCEINLINE U() : V() {}
	} Tmp;
	Tmp.D[0] = X;
	Tmp.D[1] = Y;
	return Tmp.V;
}

FORCEINLINE VectorRegister2Double MakeVectorRegister2Double(uint64 X, uint64 Y)
{
	union U
	{
		VectorRegister2Double V; uint64_t D[2];
		FORCEINLINE U() : V() {}
	} Tmp;
	Tmp.D[0] = X;
	Tmp.D[1] = Y;
	return Tmp.V;
}

/**
* Returns a vector based on 4 int32.
*
* @param X		1st int32 component
* @param Y		2nd int32 component
* @param Z		3rd int32 component
* @param W		4th int32 component
* @return		Vector of the 4 int32
*/
FORCEINLINE VectorRegister4Int MakeVectorRegisterInt(int32 X, int32 Y, int32 Z, int32 W)
{
	union U {
		VectorRegister4Int V; int32 I[4];
		FORCEINLINE U() : V() {}
	} Tmp;
	Tmp.I[0] = X;
	Tmp.I[1] = Y;
	Tmp.I[2] = Z;
	Tmp.I[3] = W;
	return Tmp.V;
}

FORCEINLINE VectorRegister4Int MakeVectorRegisterInt64(int64 X, int64 Y)
{
	union U
	{
		VectorRegister4Int V; int64 I[2];
		FORCEINLINE U() : V() {}
	} Tmp;
	Tmp.I[0] = X;
	Tmp.I[1] = Y;
	return Tmp.V;
}

// Make double register from float register
FORCEINLINE VectorRegister4Double MakeVectorRegisterDouble(VectorRegister4Float From)
{
	return VectorRegister4Double(From);
}

// Lossy conversion: double->float vector
FORCEINLINE VectorRegister4Float MakeVectorRegisterFloatFromDouble(VectorRegister4Double Vec)
{
	return vcombine_f32(vcvt_f32_f64(Vec.XY), vcvt_f32_f64(Vec.ZW));
}

/*
#define VectorPermute(Vec1, Vec2, Mask) my_perm(Vec1, Vec2, Mask)

/ ** Reads NumBytesMinusOne+1 bytes from the address pointed to by Ptr, always reading the aligned 16 bytes containing the start of Ptr, but only reading the next 16 bytes if the data straddles the boundary * /
FORCEINLINE VectorRegister4Float VectorLoadNPlusOneUnalignedBytes(const void* Ptr, int NumBytesMinusOne)
{
	return VectorPermute( my_ld (0, (float*)Ptr), my_ld(NumBytesMinusOne, (float*)Ptr), my_lvsl(0, (float*)Ptr) );
}
*/


/*=============================================================================
 *	Constants:
 *============================================================================*/

#include "Math/UnrealMathVectorConstants.h.inl"


/*=============================================================================
 *	Intrinsics:
 *============================================================================*/

/**
 * Returns a vector with all zeros.
 *
 * @return		VectorRegister4Float(0.0f, 0.0f, 0.0f, 0.0f)
 */
FORCEINLINE VectorRegister4Float VectorZeroFloat()
{	
	return vdupq_n_f32( 0.0f );
}

FORCEINLINE VectorRegister4Double VectorZeroDouble()
{
	VectorRegister2Double Zero = vdupq_n_f64(0.0);
	return VectorRegister4Double(Zero, Zero);
}


/**
 * Returns a vector with all ones.
 *
 * @return		VectorRegister4Float(1.0f, 1.0f, 1.0f, 1.0f)
 */
FORCEINLINE VectorRegister4Float VectorOneFloat()
{
	return vdupq_n_f32( 1.0f );
}

FORCEINLINE VectorRegister4Double VectorOneDouble()
{
	VectorRegister4Double Result;
	Result.XY = vdupq_n_f64(1.0f);
	Result.ZW = Result.XY;
	return Result;
}

/**
 * Loads 4 floats from unaligned memory.
 *
 * @param Ptr	Unaligned memory pointer to the 4 floats
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[2], Ptr[3])
 */
FORCEINLINE VectorRegister4Float VectorLoad(const float* Ptr)
{
	return vld1q_f32( (float32_t*)Ptr );
}

FORCEINLINE VectorRegister4Double VectorLoad(const double* Ptr)
{
	float64x2x2_t Vec = vld1q_f64_x2(Ptr);
	VectorRegister4Double Result = *(VectorRegister4Double*)&Vec;
	return Result;
}

/**
 * Loads 16 floats from unaligned memory into 4 vector registers.
 *
 * @param Ptr	Unaligned memory pointer to the 4 floats
 * @return		VectorRegister4x4Float containing 16 floats
 */
FORCEINLINE VectorRegister4x4Float VectorLoad16(const float* Ptr)
{
	return vld1q_f32_x4(Ptr);
}

/**
 * Loads 2 floats from unaligned memory into X and Y and duplicates them in Z and W.
 *
 * @param Ptr	Unaligned memory pointer to the floats
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[0], Ptr[1])
 */
FORCEINLINE VectorRegister4Float VectorLoadFloat2(const float* Ptr)
{
	return MakeVectorRegister(Ptr[0], Ptr[1], Ptr[0], Ptr[1]);
}

/**
 * Loads 3 floats from unaligned memory and leaves W undefined.
 *
 * @param Ptr	Unaligned memory pointer to the 3 floats
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[2], 0.0f)
 */
FORCEINLINE VectorRegister4Double VectorLoadFloat3(const double* Ptr)
{
	VectorRegister4Double Result;
	Result.XY = vld1q_f64(Ptr);
	Result.ZW = vcombine_f64(vld1_f64(&Ptr[2]), vdup_n_f64(0.0));
	return Result;
}

/**
 * Loads 3 FLOATs from unaligned memory and sets W=1.
 *
 * @param Ptr	Unaligned memory pointer to the 3 FLOATs
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[2], 1.0f)
 */
FORCEINLINE VectorRegister4Double VectorLoadFloat3_W1(const double* Ptr)
{
	VectorRegister4Double Result;
	Result.XY = vld1q_f64(Ptr);
	Result.ZW = vcombine_f64(vld1_f64(&Ptr[2]), vdup_n_f64(1.0));
	return Result;
}

/**
 * Sets a single component of a vector. Must be a define since ElementIndex needs to be a constant integer
 */
template <int ElementIndex>
FORCEINLINE VectorRegister4Float VectorSetComponentImpl(VectorRegister4Float Vec, float Scalar)
{
	return vsetq_lane_f32(Scalar, Vec, ElementIndex);
}

template <int ElementIndex>
FORCEINLINE VectorRegister2Double VectorSetComponentImpl(VectorRegister2Double Vec, double Scalar)
{
	return vsetq_lane_f64(Scalar, Vec, ElementIndex);
}

template <int ElementIndex>
FORCEINLINE VectorRegister4Double VectorSetComponentImpl(VectorRegister4Double Vec, double Scalar)
{
	VectorRegister4Double Result;
	if constexpr (ElementIndex > 1)
	{
		Result.XY = Vec.XY;
		Result.ZW = VectorSetComponentImpl<ElementIndex - 2>(Vec.ZW, Scalar);
	}
	else
	{
		Result.XY = VectorSetComponentImpl<ElementIndex>(Vec.XY, Scalar);
		Result.ZW = Vec.ZW;
	}
	return Result;
}

#define VectorSetComponent( Vec, ElementIndex, Scalar ) VectorSetComponentImpl<ElementIndex>(Vec, Scalar)


/**
 * Loads 4 floats from aligned memory.
 *
 * @param Ptr	Aligned memory pointer to the 4 floats
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[2], Ptr[3])
 */
FORCEINLINE VectorRegister4Float VectorLoadAligned(const float* Ptr)
{
	return VectorLoad(Ptr);
}

FORCEINLINE VectorRegister4Double VectorLoadAligned(const double* Ptr)
{
	return VectorLoad(Ptr);
}

/**
 * Loads 1 float from unaligned memory and replicates it to all 4 elements.
 *
 * @param Ptr	Unaligned memory pointer to the float
 * @return		VectorRegister4Float(Ptr[0], Ptr[0], Ptr[0], Ptr[0])
 */
FORCEINLINE VectorRegister4Float VectorLoadFloat1(const float *Ptr)
{
	return vdupq_n_f32(Ptr[0]);
}

FORCEINLINE VectorRegister4Double VectorLoadDouble1(const double* Ptr)
{
	VectorRegister4Double Result;
	Result.XY = vdupq_n_f64(Ptr[0]);
	Result.ZW = Result.XY;
	return Result;
}

FORCEINLINE VectorRegister4i VectorLoad64Bits(const void *Ptr)
{
	return vcombine_s64(vld1_s64((const int64_t *)Ptr), vdup_n_s64(0));
}

/**
 * Loads 4 unaligned floats - 2 from the first pointer, 2 from the second, and packs
 * them in to 1 vector.
 *
 * @param Ptr1	Unaligned memory pointer to the first 2 floats
 * @param Ptr2	Unaligned memory pointer to the second 2 floats
 * @return		VectorRegister4Float(Ptr1[0], Ptr1[1], Ptr2[0], Ptr2[1])
 */
FORCEINLINE VectorRegister4Float VectorLoadTwoPairsFloat(const float* Ptr1, const float* Ptr2)
{
	float32x2_t Lo = vld1_f32(Ptr1);
	float32x2_t Hi = vld1_f32(Ptr2);
	return vcombine_f32(Lo, Hi);
}

FORCEINLINE VectorRegister4Double VectorLoadTwoPairsFloat(const double* Ptr1, const double* Ptr2)
{
	VectorRegister4Double Res;
	Res.XY = vld1q_f64(Ptr1);
	Res.ZW = vld1q_f64(Ptr2);
	return Res;
}

/**
* Propagates passed in float to all registers.
*
* @param X		float component
* @return		VectorRegister4Float(X, X, X, X)
*/
FORCEINLINE VectorRegister4Float VectorSetFloat1(float X)
{
	return vdupq_n_f32(X);
}

FORCEINLINE VectorRegister4Double VectorSetFloat1(double X)
{
	VectorRegister4Double Result;
	Result.XY = vdupq_n_f64(X);
	Result.ZW = Result.XY;
	return Result;
}

/**
 * Stores a vector to aligned memory.
 *
 * @param Vec	Vector to store
 * @param Ptr	Aligned memory pointer
 */
FORCEINLINE void VectorStoreAligned(VectorRegister4Float Vec, float* Ptr)
{
	vst1q_f32(Ptr, Vec);
}

FORCEINLINE void VectorStoreAligned(VectorRegister4Double Vec, double* Ptr)
{
	vst1q_f64_x2(Ptr, *(float64x2x2_t*)&Vec);
}

//TODO: LWC VectorVM.cpp calls it on a line 3294, case EVectorVMOp::outputdata_half: Context.WriteExecFunction(CopyConstantToOutput<float, FFloat16, 2>); break;
FORCEINLINE void VectorStoreAligned(VectorRegister4Float Vec, FFloat16* Ptr)
{
	AlignedFloat4 Floats(Vec);
	for (int i = 0; i < 4; ++i)
	{
		Ptr[i] = Floats[i];
	}
}

/**
* Same as VectorStoreAligned for Neon. 
*
* @param Vec	Vector to store
* @param Ptr	Aligned memory pointer
*/
#define VectorStoreAlignedStreamed(Vec, Ptr)	VectorStoreAligned(Vec, Ptr)

/**
 * Stores a vector to memory (aligned or unaligned).
 *
 * @param Vec	Vector to store
 * @param Ptr	Memory pointer
 */
FORCEINLINE void VectorStore(VectorRegister4Float Vec, float* Ptr)
{
	vst1q_f32(Ptr, Vec);
}

FORCEINLINE void VectorStore(VectorRegister4Double Vec, double* Ptr)
{
	vst1q_f64_x2(Ptr, *(float64x2x2_t*)&Vec);
}

/**
 * Stores 4 vectors to memory (aligned or unaligned).
 *
 * @param Vec	Vector to store
 * @param Ptr	Memory pointer
 */
FORCEINLINE void VectorStore16(VectorRegister4x4Float Vec, float* Ptr)
{
	vst1q_f32_x4(Ptr, Vec);
}

/**
 * Stores the XYZ components of a vector to unaligned memory.
 *
 * @param Vec	Vector to store XYZ
 * @param Ptr	Unaligned memory pointer
 */
FORCEINLINE void VectorStoreFloat3(VectorRegister4Float Vec, float* Ptr)
{
	vst1_f32(Ptr, *(float32x2_t*)&Vec);
	vst1q_lane_f32(((float32_t*)Ptr) + 2, Vec, 2);
}

/**
 * Stores the XYZ components of a double vector pair to unaligned memory.
 *
 * @param Vec	Vector to store XYZ
 * @param Ptr	Unaligned memory pointer
 */
FORCEINLINE void VectorStoreFloat3(VectorRegister4Double Vec, double* Ptr)
{
	vst1q_f64(Ptr, Vec.XY);
	vst1q_lane_f64(((float64_t*)Ptr) + 2, Vec.ZW, 0);
}


/**
 * Stores the X component of a vector to unaligned memory.
 *
 * @param Vec	Vector to store X
 * @param Ptr	Unaligned memory pointer
 */
FORCEINLINE void VectorStoreFloat1(VectorRegister4Float Vec, float* Ptr)
{
	vst1q_lane_f32( Ptr, Vec, 0 );
}

FORCEINLINE void VectorStoreFloat1(VectorRegister4Double Vec, double* Ptr)
{
	vst1q_lane_f64(Ptr, Vec.XY, 0);
}

/**
 * Replicates one element into all four elements and returns the new vector. Must be a #define for ELementIndex
 * to be a constant integer
 *
 * @param Vec			Source vector
 * @param ElementIndex	Index (0-3) of the element to replicate
 * @return				VectorRegister4Float( Vec[ElementIndex], Vec[ElementIndex], Vec[ElementIndex], Vec[ElementIndex] )
 */
template <int ElementIndex>
FORCEINLINE VectorRegister4Float VectorReplicateImpl(VectorRegister4Float Vec)
{
	return vdupq_n_f32(vgetq_lane_f32(Vec, ElementIndex));
}

template <int ElementIndex>
FORCEINLINE VectorRegister2Double VectorReplicateImpl(VectorRegister2Double Vec)
{
	return vdupq_n_f64(vgetq_lane_f64(Vec, ElementIndex));
}

template <int ElementIndex>
FORCEINLINE VectorRegister4Double VectorReplicateImpl(VectorRegister4Double Vec)
{
	VectorRegister4Double Result;
	if constexpr (ElementIndex <= 1)
	{
		Result.XY = VectorReplicateImpl<ElementIndex>(Vec.XY);
		Result.ZW = Result.XY;
	}
	else
	{
		Result.ZW = VectorReplicateImpl<ElementIndex - 2>(Vec.ZW);
		Result.XY = Result.ZW;
	}
	return Result;
}

#define VectorReplicate( Vec, ElementIndex ) VectorReplicateImpl<ElementIndex>(Vec)


/**
 * Returns the absolute value (component-wise).
 *
 * @param Vec			Source vector
 * @return				VectorRegister4Float( abs(Vec.x), abs(Vec.y), abs(Vec.z), abs(Vec.w) )
 */
FORCEINLINE VectorRegister4Float VectorAbs(VectorRegister4Float Vec)
{
	return vabsq_f32( Vec );
}

FORCEINLINE VectorRegister4Double VectorAbs(VectorRegister4Double Vec)
{
	VectorRegister4Double Result;
	Result.XY = vabsq_f64(Vec.XY);
	Result.ZW = vabsq_f64(Vec.ZW);
	return Result;
}

/**
 * Returns the negated value (component-wise).
 *
 * @param Vec			Source vector
 * @return				VectorRegister4Float( -Vec.x, -Vec.y, -Vec.z, -Vec.w )
 */
FORCEINLINE VectorRegister4Float VectorNegate(VectorRegister4Float Vec)
{
	return vnegq_f32( Vec );
}

FORCEINLINE VectorRegister4Double VectorNegate(VectorRegister4Double Vec)
{
	VectorRegister4Double Result;
	Result.XY = vnegq_f64(Vec.XY);
	Result.ZW = vnegq_f64(Vec.ZW);
	return Result;
}

/**
 * Adds two vectors (component-wise) and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x+Vec2.x, Vec1.y+Vec2.y, Vec1.z+Vec2.z, Vec1.w+Vec2.w )
 */
FORCEINLINE VectorRegister4Float VectorAdd(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return vaddq_f32( Vec1, Vec2 );
}

FORCEINLINE VectorRegister4Double VectorAdd(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
	Result.XY = vaddq_f64(Vec1.XY, Vec2.XY);
	Result.ZW = vaddq_f64(Vec1.ZW, Vec2.ZW);
	return Result;
}


/**
 * Subtracts a vector from another (component-wise) and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x-Vec2.x, Vec1.y-Vec2.y, Vec1.z-Vec2.z, Vec1.w-Vec2.w )
 */
FORCEINLINE VectorRegister4Float VectorSubtract(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return vsubq_f32( Vec1, Vec2 );
}

FORCEINLINE VectorRegister4Double VectorSubtract(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Res;
	Res.XY = vsubq_f64(Vec1.XY, Vec2.XY);
	Res.ZW = vsubq_f64(Vec1.ZW, Vec2.ZW);
	return Res;
}


/**
 * Multiplies two vectors (component-wise) and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x*Vec2.x, Vec1.y*Vec2.y, Vec1.z*Vec2.z, Vec1.w*Vec2.w )
 */
FORCEINLINE VectorRegister4Float VectorMultiply(VectorRegister4Float Vec1, VectorRegister4Float Vec2) 
{
	return vmulq_f32( Vec1, Vec2 );
}

FORCEINLINE VectorRegister2Double VectorMultiply(VectorRegister2Double Vec1, VectorRegister2Double Vec2)
{
	return vmulq_f64(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorMultiply(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
	Result.XY = vmulq_f64(Vec1.XY, Vec2.XY);
	Result.ZW = vmulq_f64(Vec1.ZW, Vec2.ZW);
	return Result;
}


/**
* Divides two vectors (component-wise) and returns the result.
*
* @param Vec1	1st vector
* @param Vec2	2nd vector
* @return		VectorRegister4Float( Vec1.x/Vec2.x, Vec1.y/Vec2.y, Vec1.z/Vec2.z, Vec1.w/Vec2.w )
*/
FORCEINLINE VectorRegister4Float VectorDivide(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return vdivq_f32(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorDivide(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Res;
	Res.XY = vdivq_f64(Vec1.XY, Vec2.XY);
	Res.ZW = vdivq_f64(Vec1.ZW, Vec2.ZW);
	return Res;
}


/**
 * Multiplies two vectors (component-wise), adds in the third vector and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @param Acc	3rd vector
 * @return		VectorRegister4Float( Vec1.x*Vec2.x + Acc.x, Vec1.y*Vec2.y + Acc.y, Vec1.z*Vec2.z + Acc.z, Vec1.w*Vec2.w + Acc.w )
 */
FORCEINLINE VectorRegister4Float VectorMultiplyAdd(VectorRegister4Float Vec1, VectorRegister4Float Vec2, VectorRegister4Float Acc)
{
	return vfmaq_f32(Acc, Vec1, Vec2 );
}

FORCEINLINE VectorRegister4Double VectorMultiplyAdd(VectorRegister4Double Vec1, VectorRegister4Double Vec2, VectorRegister4Double Acc)
{
	VectorRegister4Double Result;
	Result.XY = vfmaq_f64(Acc.XY, Vec1.XY, Vec2.XY);
	Result.ZW = vfmaq_f64(Acc.ZW, Vec1.ZW, Vec2.ZW);
	return Result;
}

/**
 * Multiplies two vectors (component-wise) and subtracts the result from the third vector.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @param Sub	3rd vector
 * @return		VectorRegister4Float( Sub.x - Vec1.x*Vec2.x, Sub.y - Vec1.y*Vec2.y, Sub.z - Vec1.z*Vec2.z, Sub.w - Vec1.w*Vec2.w )
 */
FORCEINLINE VectorRegister4Float VectorNegateMultiplyAdd(VectorRegister4Float Vec1, VectorRegister4Float Vec2, VectorRegister4Float Sub)
{
	return vfmsq_f32(Sub, Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorNegateMultiplyAdd(VectorRegister4Double Vec1, VectorRegister4Double Vec2, VectorRegister4Double Sub)
{
	VectorRegister4Double Result;
	Result.XY = vfmsq_f64(Sub.XY, Vec1.XY, Vec2.XY);
	Result.ZW = vfmsq_f64(Sub.ZW, Vec1.ZW, Vec2.ZW);
	return Result;
}


/**
 * Calculates the dot3 product of two vectors and returns a vector with the result in all 4 components.
 * Only really efficient on Xbox 360.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		d = dot3(Vec1.xyz, Vec2.xyz), VectorRegister4Float( d, d, d, d )
 */
FORCEINLINE VectorRegister4Float VectorDot3(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	VectorRegister4Float Temp = VectorMultiply( Vec1, Vec2 );
	Temp = vsetq_lane_f32( 0.0f, Temp, 3 );
	float32x2_t sum = vpadd_f32( vget_low_f32( Temp ), vget_high_f32( Temp ) );
	sum = vpadd_f32( sum, sum );
	return vdupq_lane_f32( sum, 0 );
}

FORCEINLINE VectorRegister4Double VectorDot3(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister2Double A, B;
	A = vmulq_f64(Vec1.XY, Vec2.XY);
	B = vfmaq_f64(A, Vec1.ZW, Vec2.ZW);
	float64x1_t Sum = vadd_f64(vget_low_f64(B), vget_high_f64(A));
	VectorRegister4Double Temp;
	Temp.XY = vdupq_lane_f64(Sum, 0);
	Temp.ZW = Temp.XY;
	return Temp;
}

FORCEINLINE float VectorDot3Scalar(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return vgetq_lane_f32(VectorDot3(Vec1, Vec2), 0);
}

FORCEINLINE double VectorDot3Scalar(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister2Double A, B;
	A = vmulq_f64(Vec1.XY, Vec2.XY);
	B = vfmaq_f64(A, Vec1.ZW, Vec2.ZW);
	float64x1_t Sum = vadd_f64(vget_low_f64(B), vget_high_f64(A));
	return *(double*)&Sum;
}



/**
 * Calculates the dot4 product of two vectors and returns a vector with the result in all 4 components.
 * Only really efficient on Xbox 360.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		d = dot4(Vec1.xyzw, Vec2.xyzw), VectorRegister4Float( d, d, d, d )
 */
FORCEINLINE VectorRegister4Float VectorDot4(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	VectorRegister4Float Temp = VectorMultiply(Vec1, Vec2);
	float32x2_t sum = vpadd_f32(vget_low_f32(Temp), vget_high_f32(Temp));
	sum = vpadd_f32(sum, sum);
	return vdupq_lane_f32(sum, 0);
}

FORCEINLINE VectorRegister4Double VectorDot4(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister2Double A, B;
	A = vmulq_f64(Vec1.XY, Vec2.XY);
	B = vfmaq_f64(A, Vec1.ZW, Vec2.ZW);
	A = vextq_f64(B, B, 1);
	VectorRegister4Double Temp;
	Temp.XY = vaddq_f64(A, B);
	Temp.ZW = Temp.XY;
	return Temp;
}

/**
 * Creates a four-part mask based on component-wise == compares of the input vectors
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x == Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
 */

FORCEINLINE VectorRegister4Float VectorCompareEQ(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return (VectorRegister4Float)vceqq_f32( Vec1, Vec2 );
}

FORCEINLINE VectorRegister4Double VectorCompareEQ(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
	Result.XY = (VectorRegister2Double)vceqq_f64(Vec1.XY, Vec2.XY);
	Result.ZW = (VectorRegister2Double)vceqq_f64(Vec1.ZW, Vec2.ZW);
	return Result;
}



/**
 * Creates a four-part mask based on component-wise != compares of the input vectors
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x != Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
 */

FORCEINLINE VectorRegister4Float VectorCompareNE(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return (VectorRegister4Float)vmvnq_u32( vceqq_f32( Vec1, Vec2 ) );
}

FORCEINLINE VectorRegister4Double VectorCompareNE(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
	Result.XY = (VectorRegister2Double)vmvnq_u32(vceqq_f64(Vec1.XY, Vec2.XY));
	Result.ZW = (VectorRegister2Double)vmvnq_u32(vceqq_f64(Vec1.ZW, Vec2.ZW));
	return Result;
}

/**
 * Creates a four-part mask based on component-wise > compares of the input vectors
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x > Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
 */

FORCEINLINE VectorRegister4Float VectorCompareGT(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return (VectorRegister4Float)vcgtq_f32( Vec1, Vec2 );
}

FORCEINLINE VectorRegister4Double VectorCompareGT(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
	Result.XY = (VectorRegister2Double)vcgtq_f64(Vec1.XY, Vec2.XY);
	Result.ZW = (VectorRegister2Double)vcgtq_f64(Vec1.ZW, Vec2.ZW);
	return Result;
}

/**
 * Creates a four-part mask based on component-wise >= compares of the input vectors
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x >= Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
 */

FORCEINLINE VectorRegister4Float VectorCompareGE(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return (VectorRegister4Float)vcgeq_f32( Vec1, Vec2 );
}

FORCEINLINE VectorRegister4Double VectorCompareGE(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
	Result.XY = (VectorRegister2Double)vcgeq_f64(Vec1.XY, Vec2.XY);
	Result.ZW = (VectorRegister2Double)vcgeq_f64(Vec1.ZW, Vec2.ZW);
	return Result;
}

/**
* Creates a four-part mask based on component-wise < compares of the input vectors
*
* @param Vec1	1st vector
* @param Vec2	2nd vector
* @return		VectorRegister4Float( Vec1.x < Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
*/
FORCEINLINE VectorRegister4Float VectorCompareLT(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return (VectorRegister4Float)vcltq_f32(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorCompareLT(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Res;
	Res.XY = (VectorRegister2Double)vcltq_f64(Vec1.XY, Vec2.XY);
	Res.ZW = (VectorRegister2Double)vcltq_f64(Vec1.ZW, Vec2.ZW);
	return Res;
}

/**
* Creates a four-part mask based on component-wise <= compares of the input vectors
*
* @param Vec1	1st vector
* @param Vec2	2nd vector
* @return		VectorRegister4Float( Vec1.x <= Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
*/
FORCEINLINE VectorRegister4Float VectorCompareLE(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return (VectorRegister4Float)vcleq_f32(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorCompareLE(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Res;
	Res.XY = (VectorRegister2Double)vcleq_f64(Vec1.XY, Vec2.XY);
	Res.ZW = (VectorRegister2Double)vcleq_f64(Vec1.ZW, Vec2.ZW);
	return Res;
}

/**
 * Does a bitwise vector selection based on a mask (e.g., created from VectorCompareXX)
 *
 * @param Mask  Mask (when 1: use the corresponding bit from Vec1 otherwise from Vec2)
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( for each bit i: Mask[i] ? Vec1[i] : Vec2[i] )
 *
 */

FORCEINLINE VectorRegister4Float VectorSelect(VectorRegister4Float Mask, VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return vbslq_f32((VectorRegister4Int)Mask, Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorSelect(VectorRegister4Double Mask, VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
	Result.XY = vbslq_f64((VectorRegister2Int64)Mask.XY, Vec1.XY, Vec2.XY);
	Result.ZW = vbslq_f64((VectorRegister2Int64)Mask.ZW, Vec1.ZW, Vec2.ZW);
	return Result;
}

/**
 * Combines two vectors using bitwise OR (treating each vector as a 128 bit field)
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( for each bit i: Vec1[i] | Vec2[i] )
 */
FORCEINLINE VectorRegister4Float VectorBitwiseOr(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return (VectorRegister4Float)vorrq_u32( (VectorRegister4Int)Vec1, (VectorRegister4Int)Vec2 );
}

FORCEINLINE VectorRegister4Double VectorBitwiseOr(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
	Result.XY = (VectorRegister2Double)vorrq_u64((VectorRegister2Int64)Vec1.XY, (VectorRegister2Int64)Vec2.XY);
	Result.ZW = (VectorRegister2Double)vorrq_u64((VectorRegister2Int64)Vec1.ZW, (VectorRegister2Int64)Vec2.ZW);
	return Result;
}

/**
 * Combines two vectors using bitwise AND (treating each vector as a 128 bit field)
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( for each bit i: Vec1[i] & Vec2[i] )
 */
FORCEINLINE VectorRegister4Float VectorBitwiseAnd(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return (VectorRegister4Float)vandq_u32( (VectorRegister4Int)Vec1, (VectorRegister4Int)Vec2 );
}

FORCEINLINE VectorRegister4Double VectorBitwiseAnd(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
	Result.XY = (VectorRegister2Double)vandq_u64((VectorRegister2Int64)Vec1.XY, (VectorRegister2Int64)Vec2.XY);
	Result.ZW = (VectorRegister2Double)vandq_u64((VectorRegister2Int64)Vec1.ZW, (VectorRegister2Int64)Vec2.ZW);
	return Result;
}

/**
 * Combines two vectors using bitwise XOR (treating each vector as a 128 bit field)
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( for each bit i: Vec1[i] ^ Vec2[i] )
 */
FORCEINLINE VectorRegister4Float VectorBitwiseXor(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return (VectorRegister4Float)veorq_u32( (VectorRegister4Int)Vec1, (VectorRegister4Int)Vec2 );
}

FORCEINLINE VectorRegister4Double VectorBitwiseXor(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
	Result.XY = (VectorRegister2Double)veorq_u64((VectorRegister2Int64)Vec1.XY, (VectorRegister2Int64)Vec2.XY);
	Result.ZW = (VectorRegister2Double)veorq_u64((VectorRegister2Int64)Vec1.ZW, (VectorRegister2Int64)Vec2.ZW);
	return Result;
}


/**
 * Swizzles the 4 components of a vector and returns the result.
 *
 * @param Vec		Source vector
 * @param X			Index for which component to use for X (literal 0-3)
 * @param Y			Index for which component to use for Y (literal 0-3)
 * @param Z			Index for which component to use for Z (literal 0-3)
 * @param W			Index for which component to use for W (literal 0-3)
 * @return			The swizzled vector
 */
#ifndef __clang__
FORCEINLINE VectorRegister4Float VectorSwizzle
(
	VectorRegister4Float V,
	uint32 E0,
	uint32 E1,
	uint32 E2,
	uint32 E3
)
{
	check((E0 < 4) && (E1 < 4) && (E2 < 4) && (E3 < 4));
	static constexpr uint32_t ControlElement[4] =
	{
		0x03020100, // XM_SWIZZLE_X
		0x07060504, // XM_SWIZZLE_Y
		0x0B0A0908, // XM_SWIZZLE_Z
		0x0F0E0D0C, // XM_SWIZZLE_W
	};

	uint8x8x2_t tbl;
	tbl.val[0] = vget_low_f32(V);
	tbl.val[1] = vget_high_f32(V);

	uint32x2_t idx = vcreate_u32(static_cast<uint64>(ControlElement[E0]) | (static_cast<uint64>(ControlElement[E1]) << 32));
	const uint8x8_t rL = vtbl2_u8(tbl, idx);

	idx = vcreate_u32(static_cast<uint64>(ControlElement[E2]) | (static_cast<uint64>(ControlElement[E3]) << 32));
	const uint8x8_t rH = vtbl2_u8(tbl, idx);

	return vcombine_f32(rL, rH);
}

FORCEINLINE VectorRegister4Double VectorSwizzle
(
	VectorRegister4Double V,
	uint32 E0,
	uint32 E1,
	uint32 E2,
	uint32 E3
)
{
	check((E0 < 4) && (E1 < 4) && (E2 < 4) && (E3 < 4));
	static constexpr uint64_t ControlElement[4] =
	{
		0x0706050403020100ULL, // XM_SWIZZLE_X
		0x0F0E0D0C0B0A0908ULL, // XM_SWIZZLE_Y
		0x1716151413121110ULL, // XM_SWIZZLE_Z
		0x1F1E1D1C1B1A1918ULL, // XM_SWIZZLE_W
	};

	uint8x16x2_t tbl;
	tbl.val[0] = V.XY;
	tbl.val[1] = V.ZW;

	VectorRegister4Double Result;
	uint32x4_t idx = vcombine_u64(vcreate_u64(ControlElement[E0]), vcreate_u64(ControlElement[E1]));
	Result.XY = vqtbl2q_u8(tbl, idx);

	idx = vcombine_u64(vcreate_u64(ControlElement[E2]), vcreate_u64(ControlElement[E3]));
	Result.ZW = vqtbl2q_u8(tbl, idx);

	return Result;
}
#else
template <int X, int Y, int Z, int W>
FORCEINLINE VectorRegister4Float VectorSwizzleImpl(VectorRegister4Float Vec)
{
	return __builtin_shufflevector(Vec, Vec, X, Y, Z, W);
}

template <int X, int Y>
FORCEINLINE VectorRegister2Double VectorSwizzleImpl2(VectorRegister4Double Vec)
{
	if constexpr (X <= 1)
	{
		if constexpr (Y <= 1)
		{
			return __builtin_shufflevector(Vec.XY, Vec.XY, X, Y);
		}
		else
		{
			return __builtin_shufflevector(Vec.XY, Vec.ZW, X, Y);
		}
	}
	else
	{
		if constexpr (Y <= 1)
		{
			return __builtin_shufflevector(Vec.ZW, Vec.XY, X - 2, Y + 2);
		}
		else
		{
			return __builtin_shufflevector(Vec.ZW, Vec.ZW, X - 2, Y);
		}
	}
}

template <int X, int Y, int Z, int W>
FORCEINLINE VectorRegister4Double VectorSwizzleImpl(VectorRegister4Double Vec)
{
	VectorRegister4Double Result;
	Result.XY = VectorSwizzleImpl2<X, Y>(Vec);
	Result.ZW = VectorSwizzleImpl2<Z, W>(Vec);
	return Result;
}

#define VectorSwizzle( Vec, X, Y, Z, W ) VectorSwizzleImpl<X, Y, Z, W>(Vec)
#endif // __clang__ 


/**
* Creates a vector through selecting two components from each vector via a shuffle mask.
*
* @param Vec1		Source vector1
* @param Vec2		Source vector2
* @param X			Index for which component of Vector1 to use for X (literal 0-3)
* @param Y			Index for which component of Vector1 to use for Y (literal 0-3)
* @param Z			Index for which component of Vector2 to use for Z (literal 0-3)
* @param W			Index for which component of Vector2 to use for W (literal 0-3)
* @return			The swizzled vector
*/
#ifndef __clang__
FORCEINLINE VectorRegister4Float VectorShuffle
(
	VectorRegister4Float V1,
	VectorRegister4Float V2,
	uint32 PermuteX,
	uint32 PermuteY,
	uint32 PermuteZ,
	uint32 PermuteW
)
{
	check(PermuteX <= 3 && PermuteY <= 3 && PermuteZ <= 3 && PermuteW <= 3);

	static constexpr uint32 ControlElement[8] =
	{
		0x03020100, // XM_PERMUTE_0X
		0x07060504, // XM_PERMUTE_0Y
		0x0B0A0908, // XM_PERMUTE_0Z
		0x0F0E0D0C, // XM_PERMUTE_0W
		0x13121110, // XM_PERMUTE_1X
		0x17161514, // XM_PERMUTE_1Y
		0x1B1A1918, // XM_PERMUTE_1Z
		0x1F1E1D1C, // XM_PERMUTE_1W
	};

	uint8x8x4_t tbl;
	tbl.val[0] = vget_low_f32(V1);
	tbl.val[1] = vget_high_f32(V1);
	tbl.val[2] = vget_low_f32(V2);
	tbl.val[3] = vget_high_f32(V2);

	uint32x2_t idx = vcreate_u32(static_cast<uint64>(ControlElement[PermuteX]) | (static_cast<uint64>(ControlElement[PermuteY]) << 32));
	const uint8x8_t rL = vtbl4_u8(tbl, idx);

	idx = vcreate_u32(static_cast<uint64>(ControlElement[PermuteZ + 4]) | (static_cast<uint64>(ControlElement[PermuteW + 4]) << 32));
	const uint8x8_t rH = vtbl4_u8(tbl, idx);

	return vcombine_f32(rL, rH);
}

FORCEINLINE VectorRegister4Double VectorShuffle
(
	VectorRegister4Double V1,
	VectorRegister4Double V2,
	uint32 PermuteX,
	uint32 PermuteY,
	uint32 PermuteZ,
	uint32 PermuteW
)
{
	check(PermuteX <= 3 && PermuteY <= 3 && PermuteZ <= 3 && PermuteW <= 3);

	static constexpr uint64 ControlElement[8] =
	{
		0x0706050403020100ULL, // XM_PERMUTE_0X
		0x0F0E0D0C0B0A0908ULL, // XM_PERMUTE_0Y
		0x1716151413121110ULL, // XM_PERMUTE_0Z
		0x1F1E1D1C1B1A1918ULL, // XM_PERMUTE_0W

		0x2726252423222120ULL, // XM_PERMUTE_1X
		0x2F2E2D2C2B2A2928ULL, // XM_PERMUTE_1Y
		0x3736353433323130ULL, // XM_PERMUTE_1Z
		0x3F3E3D3C3B3A3938ULL, // XM_PERMUTE_1W
	};

	uint8x16x4_t tbl;
	tbl.val[0] = V1.XY;
	tbl.val[1] = V1.ZW;
	tbl.val[2] = V2.XY;
	tbl.val[3] = V2.ZW;

	VectorRegister4Double Result;
	uint32x4_t idx = vcombine_u64(vcreate_u64(ControlElement[PermuteX]), vcreate_u64(ControlElement[PermuteY]));
	Result.XY = vqtbl4q_u8(tbl, idx);

	idx = vcombine_u64(vcreate_u64(ControlElement[PermuteZ + 4]), vcreate_u64(ControlElement[PermuteW + 4]));
	Result.ZW = vqtbl4q_u8(tbl, idx);

	return Result;
}
#else

template <int X, int Y, int Z, int W>
FORCEINLINE VectorRegister4Float VectorShuffleImpl(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return __builtin_shufflevector(Vec1, Vec2, X, Y, Z + 4, W + 4);
}

template <int X, int Y, int Z, int W>
FORCEINLINE VectorRegister4Double VectorShuffleImpl(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
	Result.XY = VectorSwizzleImpl2<X, Y>(Vec1);
	Result.ZW = VectorSwizzleImpl2<Z, W>(Vec2);
	return Result;
}

#define VectorShuffle( Vec1, Vec2, X, Y, Z, W )	VectorShuffleImpl<X, Y, Z, W>(Vec1, Vec2)
#endif // __clang__ 

/**
 * Returns an integer bit-mask (0x00 - 0x0f) based on the sign-bit for each component in a vector.
 *
 * @param VecMask		Vector
 * @return				Bit 0 = sign(VecMask.x), Bit 1 = sign(VecMask.y), Bit 2 = sign(VecMask.z), Bit 3 = sign(VecMask.w)
 */
FORCEINLINE uint32 VectorMaskBits(VectorRegister4Float VecMask)
{
	int32x4_t Signs = vshrq_n_s32(vreinterpretq_s32_f32(VecMask), 31); // sign bit of each lane replicated 32x
	int32x4_t Masked = vandq_s32(Signs, MakeVectorRegisterInt(0x1, 0x2, 0x4, 0x8)); // pick bit for lane position
	return uint32(vaddvq_s32(Masked)); // reduce via add
}

FORCEINLINE uint32 VectorMaskBits(VectorRegister4Double VecMask)
{
	int64x2_t Signs0 = vshrq_n_s64(vreinterpretq_s64_f32(VecMask.XY), 63); // sign bit of each lane replicated 64x
	int64x2_t Signs1 = vshrq_n_s64(vreinterpretq_s64_f32(VecMask.ZW), 63); // sign bit of each lane replicated 64x
	int32x4_t Signs = vuzp1q_s32(Signs0, Signs1); // 32-bit masks
	int32x4_t Masked = vandq_s32(Signs, MakeVectorRegisterInt(0x1, 0x2, 0x4, 0x8)); // pick bit for lane position
	return uint32(vaddvq_s32(Masked)); // reduce via add
}

/**
* Creates a vector by combining two high components from each vector
*
* @param Vec1		Source vector1
* @param Vec2		Source vector2
* @return			The combined vector
*/
FORCEINLINE VectorRegister4Float VectorCombineHigh(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return vcombine_f32(vget_high_f32(Vec1), vget_high_f32(Vec2));
}

FORCEINLINE VectorRegister4Double VectorCombineHigh(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
	Result.XY = Vec1.ZW;
	Result.ZW = Vec2.ZW;
	return Result;
}

/**
* Creates a vector by combining two low components from each vector
*
* @param Vec1		Source vector1
* @param Vec2		Source vector2
* @return			The combined vector
*/
FORCEINLINE VectorRegister4Float VectorCombineLow(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return vcombine_f32(vget_low_f32(Vec1), vget_low_f32(Vec2));
}

FORCEINLINE VectorRegister4Double VectorCombineLow(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
	Result.XY = Vec1.XY;
	Result.ZW = Vec2.XY;
	return Result;
}

/**
 * Deinterleaves the components of the two given vectors such that the even components
 * are in one vector and the odds in another.
 *
 * @param Lo	[Even0, Odd0, Even1, Odd1]
 * @param Hi	[Even2, Odd2, Even3, Odd3]
 * @param OutEvens [Even0, Even1, Even2, Even3]
 * @param OutOdds [Odd0, Odd1, Odd2, Odd3]
*/
FORCEINLINE void VectorDeinterleave(VectorRegister4Float& RESTRICT OutEvens, VectorRegister4Float& RESTRICT OutOdds, VectorRegister4Float Lo, VectorRegister4Float Hi)
{
	float32x4x2_t deinterleaved = vuzpq_f32(Lo, Hi);
	OutEvens = deinterleaved.val[0];
	OutOdds = deinterleaved.val[1];
}

FORCEINLINE void VectorDeinterleave(VectorRegister4Double& RESTRICT OutEvens, VectorRegister4Double& RESTRICT OutOdds, VectorRegister4Double Lo, VectorRegister4Double Hi)
{
	OutEvens = VectorShuffle(Lo, Hi, 0, 2, 0, 2);
	OutOdds = VectorShuffle(Lo, Hi, 1, 3, 1, 3);
}

/**
 * Calculates the cross product of two vectors (XYZ components). W of the input should be 0, and will remain 0.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		cross(Vec1.xyz, Vec2.xyz). W of the input should be 0, and will remain 0.
 */
FORCEINLINE VectorRegister4Float VectorCross(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	VectorRegister4Float C = VectorMultiply(Vec1, VectorSwizzle(Vec2, 1, 2, 0, 3));
	C = VectorNegateMultiplyAdd(VectorSwizzle(Vec1, 1, 2, 0, 3), Vec2, C);
	C = VectorSwizzle(C, 1, 2, 0, 3);
	return C;
}

FORCEINLINE VectorRegister4Double VectorCross(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double C = VectorMultiply(Vec1, VectorSwizzle(Vec2, 1, 2, 0, 3));
	C = VectorNegateMultiplyAdd(VectorSwizzle(Vec1, 1, 2, 0, 3), Vec2, C);
	C = VectorSwizzle(C, 1, 2, 0, 3);
	return C;
}

/**
 * Calculates x raised to the power of y (component-wise).
 *
 * @param Base		Base vector
 * @param Exponent	Exponent vector
 * @return			VectorRegister4Float( Base.x^Exponent.x, Base.y^Exponent.y, Base.z^Exponent.z, Base.w^Exponent.w )
 */
FORCEINLINE VectorRegister4Float VectorPow(VectorRegister4Float Base, VectorRegister4Float Exponent)
{
	//@TODO: Optimize this
	union U { 
		VectorRegister4Float V; float F[4]; 
		FORCEINLINE U() : V() {}
	} B, E;
	B.V = Base;
	E.V = Exponent;
	return MakeVectorRegister( powf(B.F[0], E.F[0]), powf(B.F[1], E.F[1]), powf(B.F[2], E.F[2]), powf(B.F[3], E.F[3]) );
}

FORCEINLINE VectorRegister4Double VectorPow(VectorRegister4Double Base, VectorRegister4Double Exponent)
{
	//@TODO: Optimize this
	AlignedDouble4 Values(Base);
	AlignedDouble4 Exponents(Exponent);

	Values[0] = FMath::Pow(Values[0], Exponents[0]);
	Values[1] = FMath::Pow(Values[1], Exponents[1]);
	Values[2] = FMath::Pow(Values[2], Exponents[2]);
	Values[3] = FMath::Pow(Values[3], Exponents[3]);
	return Values.ToVectorRegister();
}

/**
 * Computes an estimate of the reciprocal of a vector (component-wise) and returns the result.
 *
 * @param Vec	1st vector
 * @return		VectorRegister4Float( (Estimate) 1.0f / Vec.x, (Estimate) 1.0f / Vec.y, (Estimate) 1.0f / Vec.z, (Estimate) 1.0f / Vec.w )
 */
FORCEINLINE VectorRegister4Float VectorReciprocalEstimate(VectorRegister4Float Vec)
{
	return vrecpeq_f32(Vec);
}

FORCEINLINE VectorRegister4Double VectorReciprocalEstimate(VectorRegister4Double Vec)
{
	VectorRegister4Double Result;
	Result.XY = vrecpeq_f64(Vec.XY);
	Result.ZW = vrecpeq_f64(Vec.ZW);
	return Result;
}


/**
 * Computes the reciprocal of a vector (component-wise) and returns the result.
 *
 * @param Vec	1st vector
 * @return		VectorRegister4Float( 1.0f / Vec.x, 1.0f / Vec.y, 1.0f / Vec.z, 1.0f / Vec.w )
 */
FORCEINLINE VectorRegister4Float VectorReciprocal(VectorRegister4Float Vec)
{
	// Perform two passes of Newton-Raphson iteration on the hardware estimate
	// The built-in instruction (VRECPS) is not as accurate

	// Initial estimate
	VectorRegister4Float Reciprocal = VectorReciprocalEstimate(Vec);

	// First iteration
	VectorRegister4Float Squared = VectorMultiply(Reciprocal, Reciprocal);
	VectorRegister4Float Double = VectorAdd(Reciprocal, Reciprocal);
	Reciprocal = VectorNegateMultiplyAdd(Vec, Squared, Double);

	// Second iteration
	Squared = VectorMultiply(Reciprocal, Reciprocal);
	Double = VectorAdd(Reciprocal, Reciprocal);
	return VectorNegateMultiplyAdd(Vec, Squared, Double);
}

FORCEINLINE VectorRegister4Double VectorReciprocal(VectorRegister4Double Vec)
{
	return VectorDivide(GlobalVectorConstants::DoubleOne, Vec);
}


/**
 * Return the square root of each component
 *
 * @param Vector	Vector
 * @return			VectorRegister4Float(sqrt(Vec.X), sqrt(Vec.Y), sqrt(Vec.Z), sqrt(Vec.W))
 */
FORCEINLINE VectorRegister4Float VectorSqrt(VectorRegister4Float Vec)
{
	return vsqrtq_f32(Vec);
}

FORCEINLINE VectorRegister4Double VectorSqrt(VectorRegister4Double Vec)
{
	VectorRegister4Double Result;
	Result.XY = vsqrtq_f64(Vec.XY);
	Result.ZW = vsqrtq_f64(Vec.ZW);
	return Result;
}

/**
 * Returns an estimate of 1/sqrt(c) for each component of the vector
 *
 * @param Vector	Vector 
 * @return			VectorRegister4Float(1/sqrt(t), 1/sqrt(t), 1/sqrt(t), 1/sqrt(t))
 */
FORCEINLINE VectorRegister4Float VectorReciprocalSqrtEstimate(VectorRegister4Float Vec)
{
	return vrsqrteq_f32(Vec);
}

FORCEINLINE VectorRegister4Double VectorReciprocalSqrtEstimate(VectorRegister4Double Vec)
{
	VectorRegister4Double Result;
	Result.XY = vrsqrteq_f64(Vec.XY);
	Result.ZW = vrsqrteq_f64(Vec.ZW);
	return Result;
}

/**
 * Return the reciprocal of the square root of each component
 *
 * @param Vector	Vector
 * @return			VectorRegister4Float(1/sqrt(Vec.X), 1/sqrt(Vec.Y), 1/sqrt(Vec.Z), 1/sqrt(Vec.W))
 */
FORCEINLINE VectorRegister4Float VectorReciprocalSqrt(VectorRegister4Float Vec)
{
	// Initial estimate
	VectorRegister4Float RecipSqrt = VectorReciprocalSqrtEstimate(Vec);

	// Two refinement
	RecipSqrt = VectorMultiply(vrsqrtsq_f32(Vec, VectorMultiply(RecipSqrt, RecipSqrt)), RecipSqrt);
	return VectorMultiply(vrsqrtsq_f32(Vec, VectorMultiply(RecipSqrt, RecipSqrt)), RecipSqrt);
}

FORCEINLINE VectorRegister4Double VectorReciprocalSqrt(VectorRegister4Double Vec)
{
	// Initial estimate
	VectorRegister4Double RecipSqrt = VectorReciprocalSqrtEstimate(Vec);

	// Two refinement
	VectorRegister4Double Tmp;
	Tmp.XY = vrsqrtsq_f64(Vec.XY, VectorMultiply(RecipSqrt.XY, RecipSqrt.XY));
	Tmp.ZW = vrsqrtsq_f64(Vec.ZW, VectorMultiply(RecipSqrt.ZW, RecipSqrt.ZW));
	RecipSqrt = VectorMultiply(Tmp, RecipSqrt);

	Tmp.XY = vrsqrtsq_f64(Vec.XY, VectorMultiply(RecipSqrt.XY, RecipSqrt.XY));
	Tmp.ZW = vrsqrtsq_f64(Vec.ZW, VectorMultiply(RecipSqrt.ZW, RecipSqrt.ZW));
	return VectorMultiply(Tmp, RecipSqrt);
}

/**
 * Return Reciprocal Length of the vector
 *
 * @param Vector	Vector
 * @return			VectorRegister4Float(rlen, rlen, rlen, rlen) when rlen = 1/sqrt(dot4(V))
 */
FORCEINLINE VectorRegister4Float VectorReciprocalLen(VectorRegister4Float Vector)
{
	return VectorReciprocalSqrt(VectorDot4(Vector, Vector));
}

FORCEINLINE VectorRegister4Double VectorReciprocalLen(VectorRegister4Double Vector)
{
	return VectorReciprocalSqrt(VectorDot4(Vector, Vector));
}

/**
 * Return Reciprocal Length of the vector (estimate)
 *
 * @param Vector	Vector
 * @return			VectorRegister4Float(rlen, rlen, rlen, rlen) when rlen = 1/sqrt(dot4(V)) (estimate)
 */
FORCEINLINE VectorRegister4Float VectorReciprocalLenEstimate(VectorRegister4Float Vector)
{
	return VectorReciprocalSqrtEstimate(VectorDot4(Vector, Vector));
}

FORCEINLINE VectorRegister4Double VectorReciprocalLenEstimate(VectorRegister4Double Vector)
{
	return VectorReciprocalSqrtEstimate(VectorDot4(Vector, Vector));
}


/**
* Loads XYZ and sets W=0
*
* @param Vector	VectorRegister4Float
* @return		VectorRegister4Float(X, Y, Z, 0.0f)
*/
FORCEINLINE VectorRegister4Float VectorSet_W0(VectorRegister4Float Vec)
{
	return VectorSetComponent(Vec, 3, 0.0f);
}

FORCEINLINE VectorRegister4Double VectorSet_W0(VectorRegister4Double Vec)
{
	return VectorSetComponent(Vec, 3, 0.0);
}


/**
* Loads XYZ and sets W=1.
*
* @param Vector	VectorRegister4Float
* @return		VectorRegister4Float(X, Y, Z, 1.0f)
*/
FORCEINLINE VectorRegister4Float VectorSet_W1(VectorRegister4Float Vec)
{
	return VectorSetComponent(Vec, 3, 1.0f);
}

FORCEINLINE VectorRegister4Double VectorSet_W1(VectorRegister4Double Vec)
{
	return VectorSetComponent(Vec, 3, 1.0);
}



/**
* Returns a component from a vector.
*
* @param Vec				Vector register
* @param ComponentIndex	Which component to get, X=0, Y=1, Z=2, W=3
* @return					The component as a float
*/
template <uint32 ElementIndex>
FORCEINLINE float VectorGetComponentImpl(VectorRegister4Float Vec)
{
	return vgetq_lane_f32(Vec, ElementIndex);
}

template <int ElementIndex>
FORCEINLINE double VectorGetComponentImpl(VectorRegister2Double Vec)
{
	return vgetq_lane_f64(Vec, ElementIndex);
}

template <int ElementIndex>
FORCEINLINE double VectorGetComponentImpl(VectorRegister4Double Vec)
{
	if constexpr (ElementIndex > 1)
	{
		return VectorGetComponentImpl<ElementIndex - 2>(Vec.ZW);
	}
	else
	{
		return VectorGetComponentImpl<ElementIndex>(Vec.XY);
	}
}

#define VectorGetComponent(Vec, ElementIndex) VectorGetComponentImpl<ElementIndex>(Vec)

FORCEINLINE float VectorGetComponentDynamic(VectorRegister4Float Vec, uint32 ElementIndex)
{
	AlignedFloat4 Floats(Vec);
	return Floats[ElementIndex];
}

FORCEINLINE double VectorGetComponentDynamic(VectorRegister4Double Vec, uint32 ElementIndex)
{
	AlignedDouble4 Doubles(Vec);
	return Doubles[ElementIndex];
}

/**
 * Multiplies two 4x4 matrices.
 *
 * @param Result	Pointer to where the result should be stored
 * @param Matrix1	Pointer to the first matrix
 * @param Matrix2	Pointer to the second matrix
 */
FORCEINLINE void VectorMatrixMultiply(FMatrix44f* Result, const FMatrix44f* Matrix1, const FMatrix44f* Matrix2)
{
	float32x4x4_t A = vld1q_f32_x4((const float*)Matrix1);
	float32x4x4_t B = vld1q_f32_x4((const float*)Matrix2);
	float32x4x4_t R;

	// First row of result (Matrix1[0] * Matrix2).
	R.val[0] = vmulq_lane_f32(B.val[0], vget_low_f32(A.val[0]), 0);
	R.val[0] = vfmaq_lane_f32(R.val[0], B.val[1], vget_low_f32(A.val[0]), 1);
	R.val[0] = vfmaq_lane_f32(R.val[0], B.val[2], vget_high_f32(A.val[0]), 0);
	R.val[0] = vfmaq_lane_f32(R.val[0], B.val[3], vget_high_f32(A.val[0]), 1);

	// Second row of result (Matrix1[1] * Matrix2).
	R.val[1] = vmulq_lane_f32(B.val[0], vget_low_f32(A.val[1]), 0);
	R.val[1] = vfmaq_lane_f32(R.val[1], B.val[1], vget_low_f32(A.val[1]), 1);
	R.val[1] = vfmaq_lane_f32(R.val[1], B.val[2], vget_high_f32(A.val[1]), 0);
	R.val[1] = vfmaq_lane_f32(R.val[1], B.val[3], vget_high_f32(A.val[1]), 1);

	// Third row of result (Matrix1[2] * Matrix2).
	R.val[2] = vmulq_lane_f32(B.val[0], vget_low_f32(A.val[2]), 0);
	R.val[2] = vfmaq_lane_f32(R.val[2], B.val[1], vget_low_f32(A.val[2]), 1);
	R.val[2] = vfmaq_lane_f32(R.val[2], B.val[2], vget_high_f32(A.val[2]), 0);
	R.val[2] = vfmaq_lane_f32(R.val[2], B.val[3], vget_high_f32(A.val[2]), 1);

	// Fourth row of result (Matrix1[3] * Matrix2).
	R.val[3] = vmulq_lane_f32(B.val[0], vget_low_f32(A.val[3]), 0);
	R.val[3] = vfmaq_lane_f32(R.val[3], B.val[1], vget_low_f32(A.val[3]), 1);
	R.val[3] = vfmaq_lane_f32(R.val[3], B.val[2], vget_high_f32(A.val[3]), 0);
	R.val[3] = vfmaq_lane_f32(R.val[3], B.val[3], vget_high_f32(A.val[3]), 1);

	vst1q_f32_x4((float*)Result, R);
}

FORCEINLINE void VectorMatrixMultiply(FMatrix44d* Result, const FMatrix44d* Matrix1, const FMatrix44d* Matrix2)
{
	float64x2x4_t A = vld1q_f64_x4((const double*)Matrix1);
	float64x2x4_t B1 = vld1q_f64_x4((const double*)Matrix2);
	float64x2x4_t B2 = vld1q_f64_x4((const double*)Matrix2 + 8);
	float64_t* V = (float64_t*)&A;
	float64x2x4_t R;

	// First row of result (Matrix1[0] * Matrix2).
	R.val[0] = vmulq_n_f64(B1.val[0], V[0]);
	R.val[0] = vfmaq_n_f64(R.val[0], B1.val[2], V[1]);
	R.val[0] = vfmaq_n_f64(R.val[0], B2.val[0], V[2]);
	R.val[0] = vfmaq_n_f64(R.val[0], B2.val[2], V[3]);

	R.val[1] = vmulq_n_f64(B1.val[1], V[0]);
	R.val[1] = vfmaq_n_f64(R.val[1], B1.val[3], V[1]);
	R.val[1] = vfmaq_n_f64(R.val[1], B2.val[1], V[2]);
	R.val[1] = vfmaq_n_f64(R.val[1], B2.val[3], V[3]);

	// Second row of result (Matrix1[1] * Matrix2).
	R.val[2] = vmulq_n_f64(B1.val[0], V[4]);
	R.val[2] = vfmaq_n_f64(R.val[2], B1.val[2], V[5]);
	R.val[2] = vfmaq_n_f64(R.val[2], B2.val[0], V[6]);
	R.val[2] = vfmaq_n_f64(R.val[2], B2.val[2], V[7]);

	R.val[3] = vmulq_n_f64(B1.val[1], V[4]);
	R.val[3] = vfmaq_n_f64(R.val[3], B1.val[3], V[5]);
	R.val[3] = vfmaq_n_f64(R.val[3], B2.val[1], V[6]);
	R.val[3] = vfmaq_n_f64(R.val[3], B2.val[3], V[7]);

	vst1q_f64_x4((double*)Result, R);
	A = vld1q_f64_x4((const double*)Matrix1 + 8);
	V = (float64_t*)&A;

	// Third row of result (Matrix1[2] * Matrix2).
	R.val[0] = vmulq_n_f64(B1.val[0], V[0]);
	R.val[0] = vfmaq_n_f64(R.val[0], B1.val[2], V[1]);
	R.val[0] = vfmaq_n_f64(R.val[0], B2.val[0], V[2]);
	R.val[0] = vfmaq_n_f64(R.val[0], B2.val[2], V[3]);

	R.val[1] = vmulq_n_f64(B1.val[1], V[0]);
	R.val[1] = vfmaq_n_f64(R.val[1], B1.val[3], V[1]);
	R.val[1] = vfmaq_n_f64(R.val[1], B2.val[1], V[2]);
	R.val[1] = vfmaq_n_f64(R.val[1], B2.val[3], V[3]);

	// Fourth row of result (Matrix1[3] * Matrix2).
	R.val[2] = vmulq_n_f64(B1.val[0], V[4]);
	R.val[2] = vfmaq_n_f64(R.val[2], B1.val[2], V[5]);
	R.val[2] = vfmaq_n_f64(R.val[2], B2.val[0], V[6]);
	R.val[2] = vfmaq_n_f64(R.val[2], B2.val[2], V[7]);

	R.val[3] = vmulq_n_f64(B1.val[1], V[4]);
	R.val[3] = vfmaq_n_f64(R.val[3], B1.val[3], V[5]);
	R.val[3] = vfmaq_n_f64(R.val[3], B2.val[1], V[6]);
	R.val[3] = vfmaq_n_f64(R.val[3], B2.val[3], V[7]);

	vst1q_f64_x4((double*)Result + 8, R);
}

/**
 * Calculate the inverse of an FMatrix44.  Src == Dst is allowed
 *
 * @param DstMatrix		FMatrix44 pointer to where the result should be stored
 * @param SrcMatrix		FMatrix44 pointer to the Matrix to be inversed
 * @return bool			returns false if matrix is not invertable and stores identity 
 *
 */
FORCEINLINE bool VectorMatrixInverse(FMatrix44d* DstMatrix, const FMatrix44d* SrcMatrix)
{
	return FMath::MatrixInverse(DstMatrix,SrcMatrix);
}
FORCEINLINE bool VectorMatrixInverse(FMatrix44f* DstMatrix, const FMatrix44f* SrcMatrix)
{
	return FMath::MatrixInverse(DstMatrix,SrcMatrix);
}

/**
 * Calculate Homogeneous transform.
 *
 * @param VecP			VectorRegister4Float 
 * @param MatrixM		FMatrix pointer to the Matrix to apply transform
 * @return VectorRegister4Float = VecP*MatrixM
 */
FORCEINLINE VectorRegister4Float VectorTransformVector(VectorRegister4Float VecP, const FMatrix44f* MatrixM )
{
	float32x4x4_t M = vld1q_f32_x4((const float*)MatrixM);
	VectorRegister4Float Result;

	Result = vmulq_n_f32(M.val[0], VecP[0]);
	Result = vfmaq_n_f32(Result, M.val[1], VecP[1]);
	Result = vfmaq_n_f32(Result, M.val[2], VecP[2]);
	Result = vfmaq_n_f32(Result, M.val[3], VecP[3]);

	return Result;
}

FORCEINLINE VectorRegister4Float VectorTransformVector(VectorRegister4Float VecP, const FMatrix44d* MatrixM)
{
	float64x2x4_t M1 = vld1q_f64_x4((const double*)MatrixM);
	float64x2x4_t M2 = vld1q_f64_x4(((const double*)MatrixM) + 8);
	VectorRegister4Double Result;
	VectorRegister4Double Vec(VecP);

	Result.XY = vmulq_n_f64(M1.val[0], Vec.XY[0]);
	Result.XY = vfmaq_n_f64(Result.XY, M1.val[2], Vec.XY[1]);
	Result.XY = vfmaq_n_f64(Result.XY, M2.val[0], Vec.ZW[0]);
	Result.XY = vfmaq_n_f64(Result.XY, M2.val[2], Vec.ZW[1]);

	Result.ZW = vmulq_n_f64(M1.val[1], Vec.XY[0]);
	Result.ZW = vfmaq_n_f64(Result.ZW, M1.val[3], Vec.XY[1]);
	Result.ZW = vfmaq_n_f64(Result.ZW, M2.val[1], Vec.ZW[0]);
	Result.ZW = vfmaq_n_f64(Result.ZW, M2.val[3], Vec.ZW[1]);

	return MakeVectorRegisterFloatFromDouble(Result);
}

FORCEINLINE VectorRegister4Double VectorTransformVector(VectorRegister4Double VecP, const FMatrix44d* MatrixM)
{
	float64x2x4_t M1 = vld1q_f64_x4((const double*)MatrixM);
	float64x2x4_t M2 = vld1q_f64_x4(((const double*)MatrixM) + 8);
	VectorRegister4Double Result;

	//TODO: this can be rewritten to avoid using M2 var, saves some registers
	Result.XY = vmulq_n_f64(M1.val[0], VecP.XY[0]);
	Result.XY = vfmaq_n_f64(Result.XY, M1.val[2], VecP.XY[1]);
	Result.XY = vfmaq_n_f64(Result.XY, M2.val[0], VecP.ZW[0]);
	Result.XY = vfmaq_n_f64(Result.XY, M2.val[2], VecP.ZW[1]);

	Result.ZW = vmulq_n_f64(M1.val[1], VecP.XY[0]);
	Result.ZW = vfmaq_n_f64(Result.ZW, M1.val[3], VecP.XY[1]);
	Result.ZW = vfmaq_n_f64(Result.ZW, M2.val[1], VecP.ZW[0]);
	Result.ZW = vfmaq_n_f64(Result.ZW, M2.val[3], VecP.ZW[1]);

	return Result;
}

/**
 * Returns the minimum values of two vectors (component-wise).
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( min(Vec1.x,Vec2.x), min(Vec1.y,Vec2.y), min(Vec1.z,Vec2.z), min(Vec1.w,Vec2.w) )
 */
FORCEINLINE VectorRegister4Float VectorMin(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return vminq_f32( Vec1, Vec2 );
}

FORCEINLINE VectorRegister4Double VectorMin(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
	Result.XY = vminq_f64(Vec1.XY, Vec2.XY);
	Result.ZW = vminq_f64(Vec1.ZW, Vec2.ZW);
	return Result;
}

/**
 * Returns the maximum values of two vectors (component-wise).
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( max(Vec1.x,Vec2.x), max(Vec1.y,Vec2.y), max(Vec1.z,Vec2.z), max(Vec1.w,Vec2.w) )
 */
FORCEINLINE VectorRegister4Float VectorMax(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return vmaxq_f32( Vec1, Vec2 );
}

FORCEINLINE VectorRegister4Double VectorMax(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
	Result.XY = vmaxq_f64(Vec1.XY, Vec2.XY);
	Result.ZW = vmaxq_f64(Vec1.ZW, Vec2.ZW);
	return Result;
}

/**
 * Merges the XYZ components of one vector with the W component of another vector and returns the result.
 *
 * @param VecXYZ	Source vector for XYZ_
 * @param VecW		Source register for ___W (note: the fourth component is used, not the first)
 * @return			VectorRegister4Float(VecXYZ.x, VecXYZ.y, VecXYZ.z, VecW.w)
 */
FORCEINLINE VectorRegister4Float VectorMergeVecXYZ_VecW(VectorRegister4Float VecXYZ, VectorRegister4Float VecW)
{
	return vsetq_lane_f32(vgetq_lane_f32(VecW, 3), VecXYZ, 3);
}

FORCEINLINE VectorRegister4Double VectorMergeVecXYZ_VecW(VectorRegister4Double VecXYZ, VectorRegister4Double VecW)
{
	VectorRegister4Double Res;
	Res.XY = VecXYZ.XY;
	Res.ZW = vsetq_lane_f64(vgetq_lane_f64(VecW.ZW, 1), VecXYZ.ZW, 1);
	return Res;
}

/**
 * Loads 4 uint8s from unaligned memory and converts them into 4 floats.
 *
 * @param Ptr			Unaligned memory pointer to the 4 uint8s.
 * @return				VectorRegister4Float( float(Ptr[0]), float(Ptr[1]), float(Ptr[2]), float(Ptr[3]) )
 */
FORCEINLINE VectorRegister4Float VectorLoadByte4(const void* Ptr)
{
	uint8x8_t AsUInt8 = vreinterpret_u8_u32(vld1_dup_u32((const uint32*)Ptr));
	uint16x8_t AsUInt16 = vmovl_u8(AsUInt8);
	uint32x4_t AsUInt32 = vmovl_u16(vget_low_u16(AsUInt16));
	return vcvtq_f32_u32(AsUInt32);
}

/**
* Loads 4 int8s from unaligned memory and converts them into 4 floats.
*
* @param Ptr			Unaligned memory pointer to the 4 uint8s.
* @return				VectorRegister4Float( float(Ptr[0]), float(Ptr[1]), float(Ptr[2]), float(Ptr[3]) )
*/
FORCEINLINE VectorRegister4Float VectorLoadSignedByte4(const void* Ptr)
{
	int8x8_t AsInt8 = vreinterpret_s8_u32(vld1_dup_u32((const uint32*)Ptr));
	int16x8_t AsInt16 = vmovl_s8(AsInt8);
	int32x4_t AsInt32 = vmovl_s16(vget_low_u16(AsInt16));
	return vcvtq_f32_s32(AsInt32);
}

/**
 * Loads 4 uint8s from unaligned memory and converts them into 4 floats in reversed order.
 *
 * @param Ptr			Unaligned memory pointer to the 4 uint8s.
 * @return				VectorRegister4Float( float(Ptr[3]), float(Ptr[2]), float(Ptr[1]), float(Ptr[0]) )
 */
FORCEINLINE VectorRegister4Float VectorLoadByte4Reverse(const uint8* Ptr)
{
	uint8x8_t AsUInt8 = vrev32_u8(vreinterpret_u8_u32(vld1_dup_u32((const uint32*)Ptr)));
	uint16x8_t AsUInt16 = vmovl_u8(AsUInt8);
	uint32x4_t AsUInt32 = vmovl_u16(vget_low_u16(AsUInt16));
	return vcvtq_f32_u32(AsUInt32);
}

/**
 * Converts the 4 floats in the vector to 4 uint8s, clamped to [0,255], and stores to unaligned memory.
 *
 * @param Vec			Vector containing 4 floats
 * @param Ptr			Unaligned memory pointer to store the 4 uint8s.
 */
FORCEINLINE void VectorStoreByte4(VectorRegister4Float Vec, void* Ptr)
{
	uint32x4_t AsUInt32 = vcvtq_u32_f32(Vec); // Saturates (clamps) to [0,2^32 - 1]
	uint16x4_t AsUInt16 = vqmovn_u32(AsUInt32); // Saturates further to [0,2^16 - 1]
	uint8x8_t AsUInt8 = vqmovn_u16(vcombine_u16(AsUInt16, vdup_n_u16(0))); // Saturates to [0,255]
	vst1_lane_u32((uint32_t*)Ptr, AsUInt8, 0);
}

/**
* Converts the 4 floats in the vector to 4 int8s, clamped to [-128, 127], and stores to unaligned memory.
*
* @param Vec			Vector containing 4 floats
* @param Ptr			Unaligned memory pointer to store the 4 uint8s.
*/
FORCEINLINE void VectorStoreSignedByte4(VectorRegister4Float Vec, void* Ptr)
{
	int32x4_t AsInt32 = vcvtq_s32_f32(Vec); // Saturates (clamps) to [-2^31,2^31 - 1]
	int16x4_t AsInt16 = vqmovn_s32(AsInt32); // Saturates further to [-32768,32767]
	int8x8_t AsInt8 = vqmovn_s16(vcombine_s16(AsInt16, vdup_n_s16(0))); // Saturates to [-128,127]
	vst1_lane_u32((uint32_t*)Ptr, AsInt8, 0);
}

/**
 * Converts the 4 floats in the vector to 4 fp16 and stores based off bool to [un]aligned memory.
 *
 * @param Vec			Vector containing 4 floats
 * @param Ptr			Memory pointer to store the 4 fp16's.
 */
template <bool bAligned>
FORCEINLINE void VectorStoreHalf4(VectorRegister4Float Vec, void* Ptr)
{
	float16x4_t f16x4 = vcvt_f16_f32(Vec);
	vst1_u8((uint8_t*)Ptr, f16x4);
}

/**
* Loads packed RGB10A2(4 bytes) from unaligned memory and converts them into 4 FLOATs.
*
* @param Ptr			Unaligned memory pointer to the RGB10A2(4 bytes).
* @return				VectorRegister4Float with 4 FLOATs loaded from Ptr.
*/
FORCEINLINE VectorRegister4Float VectorLoadURGB10A2N(void* Ptr)
{
	alignas(16) float V[4];
	const uint32 E = *(uint32*)Ptr;
	V[0] = float((E >> 00) & 0x3FF);
	V[1] = float((E >> 10) & 0x3FF);
	V[2] = float((E >> 20) & 0x3FF);
	V[3] = float((E >> 30) & 0x3);

	VectorRegister4Float Div = MakeVectorRegister(1.0f / 1023.0f, 1.0f / 1023.0f, 1.0f / 1023.0f, 1.0f / 3.0f);
	return VectorMultiply(MakeVectorRegister(V[0], V[1], V[2], V[3]), Div);
}

/**
* Converts the 4 FLOATs in the vector RGB10A2, clamped to [0, 1023] and [0, 3], and stores to unaligned memory.
*
* @param Vec			Vector containing 4 FLOATs
* @param Ptr			Unaligned memory pointer to store the packed RGB10A2(4 bytes).
*/
FORCEINLINE void VectorStoreURGB10A2N(VectorRegister4Float Vec, void* Ptr)
{
	union U { 
		VectorRegister4Float V; float F[4]; 
		FORCEINLINE U() : V() {}
	} Tmp;
	Tmp.V = VectorMax(Vec, VectorZeroFloat());
	Tmp.V = VectorMin(Tmp.V, VectorOneFloat());
	Tmp.V = VectorMultiply(Tmp.V, MakeVectorRegister(1023.0f, 1023.0f, 1023.0f, 3.0f));

	uint32* Out = (uint32*)Ptr;
	*Out = (uint32(Tmp.F[0]) & 0x3FF) << 00 |
		(uint32(Tmp.F[1]) & 0x3FF) << 10 |
		(uint32(Tmp.F[2]) & 0x3FF) << 20 |
		(uint32(Tmp.F[3]) & 0x003) << 30;
}

/**
 * Returns non-zero if any element in Vec1 is greater than the corresponding element in Vec2, otherwise 0.
 *
 * @param Vec1			1st source vector
 * @param Vec2			2nd source vector
 * @return				Non-zero integer if (Vec1.x > Vec2.x) || (Vec1.y > Vec2.y) || (Vec1.z > Vec2.z) || (Vec1.w > Vec2.w)
 */
FORCEINLINE int32 VectorAnyGreaterThan(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	uint32x4_t Mask = (uint32x4_t)VectorCompareGT(Vec1, Vec2);
	return vmaxvq_u32(Mask);
}

FORCEINLINE int32 VectorAnyGreaterThan(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	uint32x4_t MaskXY = (uint32x4_t)vcgtq_f64(Vec1.XY, Vec2.XY);
	uint32x4_t MaskZW = (uint32x4_t)vcgtq_f64(Vec1.ZW, Vec2.ZW);
	return vmaxvq_u32(vorrq_u32(MaskXY, MaskZW));
}

/**
 * Resets the floating point registers so that they can be used again.
 */
#define VectorResetFloatRegisters()


/**
 * Disable arm64ec FPCR warnings for the time being
 * 
 * https://learn.microsoft.com/en-us/cpp/build/arm64ec-windows-abi-conventions
 * "...Avoid directly reading, writing, or computing mappings of FPCR and FPSR. These bits may be used in the future and are subject to change."
 */
#if PLATFORM_WINDOWS_ARM64EC
	#pragma warning(push)
	#pragma warning(disable:5076) // warning C5076: read from FPCR
	#pragma warning(disable:5077) // warning C5076: write to FPCR
#endif


/**
 * Returns the control register.
 *
 * @return			The uint32 control register
 */
FORCEINLINE uint32_t VectorGetControlRegister()
{
#if PLATFORM_WINDOWS && !PLATFORM_COMPILER_CLANG
	return (uint32_t)_ReadStatusReg(ARM64_FPCR);
#else
	uint64_t Value;
	// The system register read/write instructions use 64-bit registers,
	__asm__ volatile("mrs %0, fpcr" : "=r"(Value));
	return (uint32_t)Value;
#endif
}

/**
 * Sets the control register.
 *
 * @param ControlStatus		The uint32 control status value to set
 */
FORCEINLINE void VectorSetControlRegister(uint32_t ControlStatus)
{
#if PLATFORM_WINDOWS && !PLATFORM_COMPILER_CLANG
	_WriteStatusReg(ARM64_FPCR, ControlStatus);
#else
	uint64_t State64 = ControlStatus; // instruction needs a 64b reg, but all control bits fit in the lower 32b
	__asm__ volatile("msr fpcr, %0" : : "r"(State64));
#endif
}

/**
 * Restore arm64ec FPCR warnings
 */
#if PLATFORM_WINDOWS_ARM64EC
	#pragma warning(pop)
#endif


/**
 * Control status bit to round all floating point math results towards zero.
 */
#define VECTOR_ROUND_TOWARD_ZERO		(3 << 22)

 /**
  * Denormal operands and results will be flushed to zero
  */
#define VECTOR_DENORMALS_FLUSH_TO_ZERO	(1 << 24)


/**
* Multiplies two quaternions; the order matters.
*
* Order matters when composing quaternions: C = VectorQuaternionMultiply2(A, B) will yield a quaternion C = A * B
* that logically first applies B then A to any subsequent transformation (right first, then left).
*
* @param Quat1	Pointer to the first quaternion
* @param Quat2	Pointer to the second quaternion
* @return Quat1 * Quat2
*/
FORCEINLINE VectorRegister4Float VectorQuaternionMultiply2(VectorRegister4Float Quat1, VectorRegister4Float Quat2)
{
	VectorRegister4Float Result = VectorMultiply(VectorReplicate(Quat1, 3), Quat2);
	Result = VectorMultiplyAdd( VectorMultiply(VectorReplicate(Quat1, 0), VectorSwizzle(Quat2, 3,2,1,0)), GlobalVectorConstants::QMULTI_SIGN_MASK0, Result);
	Result = VectorMultiplyAdd( VectorMultiply(VectorReplicate(Quat1, 1), VectorSwizzle(Quat2, 2,3,0,1)), GlobalVectorConstants::QMULTI_SIGN_MASK1, Result);
	Result = VectorMultiplyAdd( VectorMultiply(VectorReplicate(Quat1, 2), VectorSwizzle(Quat2, 1,0,3,2)), GlobalVectorConstants::QMULTI_SIGN_MASK2, Result);

	return Result;
}

FORCEINLINE VectorRegister4Double VectorQuaternionMultiply2(VectorRegister4Double Quat1, VectorRegister4Double Quat2)
{
	VectorRegister4Double Result = VectorMultiply(VectorReplicate(Quat1, 3), Quat2);
	Result = VectorMultiplyAdd(VectorMultiply(VectorReplicate(Quat1, 0), VectorSwizzle(Quat2, 3, 2, 1, 0)), GlobalVectorConstants::DOUBLE_QMULTI_SIGN_MASK0, Result);
	Result = VectorMultiplyAdd(VectorMultiply(VectorReplicate(Quat1, 1), VectorSwizzle(Quat2, 2, 3, 0, 1)), GlobalVectorConstants::DOUBLE_QMULTI_SIGN_MASK1, Result);
	Result = VectorMultiplyAdd(VectorMultiply(VectorReplicate(Quat1, 2), VectorSwizzle(Quat2, 1, 0, 3, 2)), GlobalVectorConstants::DOUBLE_QMULTI_SIGN_MASK2, Result);

	return Result;
}

/**
* Multiplies two quaternions; the order matters.
*
* When composing quaternions: VectorQuaternionMultiply(C, A, B) will yield a quaternion C = A * B
* that logically first applies B then A to any subsequent transformation (right first, then left).
*
* @param Result	Pointer to where the result Quat1 * Quat2 should be stored
* @param Quat1	Pointer to the first quaternion (must not be the destination)
* @param Quat2	Pointer to the second quaternion (must not be the destination)
*/
FORCEINLINE void VectorQuaternionMultiply(VectorRegister4Float* RESTRICT Result, const VectorRegister4Float* RESTRICT Quat1, const VectorRegister4Float* RESTRICT Quat2)
{
	*Result = VectorQuaternionMultiply2(*Quat1, *Quat2);
}

FORCEINLINE void VectorQuaternionMultiply(VectorRegister4Double* RESTRICT Result, const VectorRegister4Double* RESTRICT Quat1, const VectorRegister4Double* RESTRICT Quat2)
{
	*Result = VectorQuaternionMultiply2(*Quat1, *Quat2);
}

/**
* Computes the sine and cosine of each component of a Vector.
*
* @param VSinAngles	VectorRegister4Float Pointer to where the Sin result should be stored
* @param VCosAngles	VectorRegister4Float Pointer to where the Cos result should be stored
* @param VAngles VectorRegister4Float Pointer to the input angles 
*/
FORCEINLINE void VectorSinCos(VectorRegister4Float* RESTRICT VSinAngles, VectorRegister4Float* RESTRICT VCosAngles, const VectorRegister4Float* RESTRICT VAngles)
{	
	// Map to [-pi, pi]
	// X = A - 2pi * round(A/2pi)
	// Note the round(), not truncate(). In this case round() can round halfway cases using round-to-nearest-even OR round-to-nearest.

	// Quotient = round(A/2pi)
	VectorRegister4Float Quotient = VectorMultiply(*VAngles, GlobalVectorConstants::OneOverTwoPi);
	Quotient = vrndnq_f32(Quotient); // round to nearest even is the default rounding mode but that's fine here.

	// X = A - 2pi * Quotient
	VectorRegister4Float X = VectorNegateMultiplyAdd(GlobalVectorConstants::TwoPi, Quotient, *VAngles);

	// Map in [-pi/2,pi/2]
	VectorRegister4Float sign = VectorBitwiseAnd(X, GlobalVectorConstants::SignBit());
	VectorRegister4Float c = VectorBitwiseOr(GlobalVectorConstants::Pi, sign);  // pi when x >= 0, -pi when x < 0
	VectorRegister4Float absx = VectorAbs(X);
	VectorRegister4Float rflx = VectorSubtract(c, X);
	VectorRegister4Float comp = VectorCompareGT(absx, GlobalVectorConstants::PiByTwo);
	X = VectorSelect(comp, rflx, X);
	sign = VectorSelect(comp, GlobalVectorConstants::FloatMinusOne, GlobalVectorConstants::FloatOne);

	const VectorRegister4Float XSquared = VectorMultiply(X, X);

	// 11-degree minimax approximation
	//*ScalarSin = (((((-2.3889859e-08f * y2 + 2.7525562e-06f) * y2 - 0.00019840874f) * y2 + 0.0083333310f) * y2 - 0.16666667f) * y2 + 1.0f) * y;
	const VectorRegister4Float SinCoeff0 = MakeVectorRegister(1.0f, -0.16666667f, 0.0083333310f, -0.00019840874f);
	const VectorRegister4Float SinCoeff1 = MakeVectorRegister(2.7525562e-06f, -2.3889859e-08f, /*unused*/ 0.f, /*unused*/ 0.f);

	VectorRegister4Float S;
	S = VectorReplicate(SinCoeff1, 1);
	S = VectorMultiplyAdd(XSquared, S, VectorReplicate(SinCoeff1, 0));
	S = VectorMultiplyAdd(XSquared, S, VectorReplicate(SinCoeff0, 3));
	S = VectorMultiplyAdd(XSquared, S, VectorReplicate(SinCoeff0, 2));
	S = VectorMultiplyAdd(XSquared, S, VectorReplicate(SinCoeff0, 1));
	S = VectorMultiplyAdd(XSquared, S, VectorReplicate(SinCoeff0, 0));
	*VSinAngles = VectorMultiply(S, X);

	// 10-degree minimax approximation
	//*ScalarCos = sign * (((((-2.6051615e-07f * y2 + 2.4760495e-05f) * y2 - 0.0013888378f) * y2 + 0.041666638f) * y2 - 0.5f) * y2 + 1.0f);
	const VectorRegister4Float CosCoeff0 = MakeVectorRegister(1.0f, -0.5f, 0.041666638f, -0.0013888378f);
	const VectorRegister4Float CosCoeff1 = MakeVectorRegister(2.4760495e-05f, -2.6051615e-07f, /*unused*/ 0.f, /*unused*/ 0.f);

	VectorRegister4Float C;
	C = VectorReplicate(CosCoeff1, 1);
	C = VectorMultiplyAdd(XSquared, C, VectorReplicate(CosCoeff1, 0));
	C = VectorMultiplyAdd(XSquared, C, VectorReplicate(CosCoeff0, 3));
	C = VectorMultiplyAdd(XSquared, C, VectorReplicate(CosCoeff0, 2));
	C = VectorMultiplyAdd(XSquared, C, VectorReplicate(CosCoeff0, 1));
	C = VectorMultiplyAdd(XSquared, C, VectorReplicate(CosCoeff0, 0));
	*VCosAngles = VectorMultiply(C, sign);
}

// Returns true if the vector contains a component that is either NAN or +/-infinite.
inline bool VectorContainsNaNOrInfinite(VectorRegister4Float Vec)
{
	// https://en.wikipedia.org/wiki/IEEE_754-1985
	// Infinity is represented with all exponent bits set, with the correct sign bit.
	// NaN is represented with all exponent bits set, plus at least one fraction/significant bit set.
	// This means finite values will not have all exponent bits set, so check against those bits.

	union { float F; uint32 U; } InfUnion;
	InfUnion.U = 0x7F800000;
	const float Inf = InfUnion.F;
	const VectorRegister4Float FloatInfinity = MakeVectorRegister(Inf, Inf, Inf, Inf);

	// Mask off Exponent
	VectorRegister4Float ExpTest = VectorBitwiseAnd(Vec, FloatInfinity);

	// Compare to full exponent & combine resulting flags into lane 0
	const int32x4_t Table = MakeVectorRegisterIntConstant(0x0C080400, 0, 0, 0);

	uint8x16_t res = (uint8x16_t)VectorCompareEQ(ExpTest, FloatInfinity);
	// If we have all zeros, all elements are finite
	return vgetq_lane_u32((uint32x4_t)vqtbx1q_u8(res, res, Table), 0) != 0;
}

inline bool VectorContainsNaNOrInfinite(VectorRegister4Double Vec)
{
	// https://en.wikipedia.org/wiki/IEEE_754-1985
	// Infinity is represented with all exponent bits set, with the correct sign bit.
	// NaN is represented with all exponent bits set, plus at least one fraction/significant bit set.
	// This means finite values will not have all exponent bits set, so check against those bits.

	union { double F; uint64 U; } InfUnion;
	InfUnion.U = 0x7FF0000000000000ULL;
	const double Inf = InfUnion.F;
	const VectorRegister4Double DoubleInfinity = MakeVectorRegister(Inf, Inf, Inf, Inf);

	// Mask off Exponent
	VectorRegister4Double ExpTest = VectorBitwiseAnd(Vec, DoubleInfinity);

	// Compare to full exponent & combine resulting flags into lane 0
	const int32x4_t Table = MakeVectorRegisterIntConstant(0x18100800, 0, 0, 0);

	VectorRegister4Double InfTestRes = VectorCompareEQ(ExpTest, DoubleInfinity);

	// If we have all zeros, all elements are finite
	uint8x16_t ZeroVec = vdupq_n_u8(0);
	//TODO: there must be a better instruction to just get the top bits or smth
	return vgetq_lane_u32((uint32x4_t)vqtbx2q_u8(ZeroVec, *(uint8x16x2_t*)&InfTestRes, Table), 0) != 0;
}

//TODO: Vectorize
FORCEINLINE VectorRegister4Float VectorExp(VectorRegister4Float X)
{
	AlignedFloat4 Val(X);
	return MakeVectorRegister(FMath::Exp(Val[0]), FMath::Exp(Val[1]), FMath::Exp(Val[2]), FMath::Exp(Val[3]));
}

FORCEINLINE VectorRegister4Double VectorExp(VectorRegister4Double X)
{
	AlignedDouble4 Val(X);
	return MakeVectorRegister(FMath::Exp(Val[0]), FMath::Exp(Val[1]), FMath::Exp(Val[2]), FMath::Exp(Val[3]));
}

//TODO: Vectorize
FORCEINLINE VectorRegister4Float VectorExp2(VectorRegister4Float X)
{
	AlignedFloat4 Val(X);
	return MakeVectorRegister(FMath::Exp2(Val[0]), FMath::Exp2(Val[1]), FMath::Exp2(Val[2]), FMath::Exp2(Val[3]));
}

FORCEINLINE VectorRegister4Double VectorExp2(VectorRegister4Double X)
{
	AlignedDouble4 Val(X);
	return MakeVectorRegister(FMath::Exp2(Val[0]), FMath::Exp2(Val[1]), FMath::Exp2(Val[2]), FMath::Exp2(Val[3]));
}

//TODO: Vectorize
FORCEINLINE VectorRegister4Float VectorLog(VectorRegister4Float X)
{
	AlignedFloat4 Val(X);
	return MakeVectorRegister(FMath::Loge(Val[0]), FMath::Loge(Val[1]), FMath::Loge(Val[2]), FMath::Loge(Val[3]));
}

FORCEINLINE VectorRegister4Double VectorLog(VectorRegister4Double X)
{
	AlignedDouble4 Val(X);
	return MakeVectorRegister(FMath::Loge(Val[0]), FMath::Loge(Val[1]), FMath::Loge(Val[2]), FMath::Loge(Val[3]));
}

//TODO: Vectorize
FORCEINLINE VectorRegister4Float VectorLog2(VectorRegister4Float X)
{
	AlignedFloat4 Val(X);
	return MakeVectorRegister(FMath::Log2(Val[0]), FMath::Log2(Val[1]), FMath::Log2(Val[2]), FMath::Log2(Val[3]));
}

FORCEINLINE VectorRegister4Double VectorLog2(VectorRegister4Double X)
{
	AlignedDouble4 Val(X);
	return MakeVectorRegister(FMath::Log2(Val[0]), FMath::Log2(Val[1]), FMath::Log2(Val[2]), FMath::Log2(Val[3]));
}

//TODO: Vectorize
FORCEINLINE VectorRegister4Float VectorTan(VectorRegister4Float X)
{
	AlignedFloat4 Val(X);
	return MakeVectorRegister(FMath::Tan(Val[0]), FMath::Tan(Val[1]), FMath::Tan(Val[2]), FMath::Tan(Val[3]));
}

FORCEINLINE VectorRegister4Double VectorTan(VectorRegister4Double X)
{
	AlignedDouble4 Val(X);
	return MakeVectorRegister(FMath::Tan(Val[0]), FMath::Tan(Val[1]), FMath::Tan(Val[2]), FMath::Tan(Val[3]));
}

//TODO: Vectorize
FORCEINLINE VectorRegister4Float VectorASin(VectorRegister4Float X)
{
	AlignedFloat4 Val(X);
	return MakeVectorRegister(FMath::Asin(Val[0]), FMath::Asin(Val[1]), FMath::Asin(Val[2]), FMath::Asin(Val[3]));
}

FORCEINLINE VectorRegister4Double VectorASin(VectorRegister4Double X)
{
	AlignedDouble4 Val(X);
	return MakeVectorRegister(FMath::Asin(Val[0]), FMath::Asin(Val[1]), FMath::Asin(Val[2]), FMath::Asin(Val[3]));
}

//TODO: Vectorize
FORCEINLINE VectorRegister4Float VectorACos(VectorRegister4Float X)
{
	AlignedFloat4 Val(X);
	return MakeVectorRegister(FMath::Acos(Val[0]), FMath::Acos(Val[1]), FMath::Acos(Val[2]), FMath::Acos(Val[3]));
}

FORCEINLINE VectorRegister4Double VectorACos(VectorRegister4Double X)
{
	AlignedDouble4 Val(X);
	return MakeVectorRegister(FMath::Acos(Val[0]), FMath::Acos(Val[1]), FMath::Acos(Val[2]), FMath::Acos(Val[3]));
}

//TODO: Vectorize
FORCEINLINE VectorRegister4Float VectorATan(VectorRegister4Float X)
{
	AlignedFloat4 Val(X);
	return MakeVectorRegister(FMath::Atan(Val[0]), FMath::Atan(Val[1]), FMath::Atan(Val[2]), FMath::Atan(Val[3]));
}

FORCEINLINE VectorRegister4Double VectorATan(VectorRegister4Double X)
{
	AlignedDouble4 Val(X);
	return MakeVectorRegister(FMath::Atan(Val[0]), FMath::Atan(Val[1]), FMath::Atan(Val[2]), FMath::Atan(Val[3]));
}

//TODO: Vectorize
FORCEINLINE VectorRegister4Float VectorATan2(VectorRegister4Float X, VectorRegister4Float Y)
{
	AlignedFloat4 ValX(X);
	AlignedFloat4 ValY(Y);

	return MakeVectorRegister(FMath::Atan2(ValX[0], ValY[0]),
							  FMath::Atan2(ValX[1], ValY[1]),
							  FMath::Atan2(ValX[2], ValY[2]),
							  FMath::Atan2(ValX[3], ValY[3]));
}

FORCEINLINE VectorRegister4Double VectorATan2(VectorRegister4Double X, VectorRegister4Double Y)
{
	AlignedDouble4 ValX(X);
	AlignedDouble4 ValY(Y);

	return MakeVectorRegister(FMath::Atan2(ValX[0], ValY[0]),
							  FMath::Atan2(ValX[1], ValY[1]),
							  FMath::Atan2(ValX[2], ValY[2]),
							  FMath::Atan2(ValX[3], ValY[3]));
}

FORCEINLINE VectorRegister4Float VectorCeil(VectorRegister4Float X)
{
	return vrndpq_f32(X);
}

FORCEINLINE VectorRegister4Double VectorCeil(VectorRegister4Double X)
{
	VectorRegister4Double Result;
	Result.XY = vrndpq_f64(X.XY);
	Result.ZW = vrndpq_f64(X.ZW);
	return Result;
}

FORCEINLINE VectorRegister4Float VectorFloor(VectorRegister4Float X)
{
	return vrndmq_f32(X);
}

FORCEINLINE VectorRegister4Double VectorFloor(VectorRegister4Double X)
{
	VectorRegister4Double Result;
	Result.XY = vrndmq_f64(X.XY);
	Result.ZW = vrndmq_f64(X.ZW);
	return Result;
}

FORCEINLINE VectorRegister4Float VectorTruncate(VectorRegister4Float X)
{
	return vrndq_f32(X);
}

FORCEINLINE VectorRegister4Double VectorTruncate(VectorRegister4Double X)
{
	VectorRegister4Double Result;
	Result.XY = vrndq_f64(X.XY);
	Result.ZW = vrndq_f64(X.ZW);
	return Result;
}

FORCEINLINE VectorRegister4Float VectorMod(VectorRegister4Float X, VectorRegister4Float Y)
{
	// Check against invalid divisor
	VectorRegister4Float InvalidDivisorMask = VectorCompareLE(VectorAbs(Y), GlobalVectorConstants::SmallNumber);
	
	AlignedFloat4 XFloats(X), YFloats(Y);
	XFloats[0] = fmodf(XFloats[0], YFloats[0]);
	XFloats[1] = fmodf(XFloats[1], YFloats[1]);
	XFloats[2] = fmodf(XFloats[2], YFloats[2]);
	XFloats[3] = fmodf(XFloats[3], YFloats[3]);
	VectorRegister4Float Result = XFloats.ToVectorRegister();

	// Return 0 where divisor Y was too small	
	Result = VectorSelect(InvalidDivisorMask, GlobalVectorConstants::FloatZero, Result);
	return Result;
}

FORCEINLINE VectorRegister4Double VectorMod(VectorRegister4Double X, VectorRegister4Double Y)
{
	// Check against invalid divisor
	VectorRegister4Double InvalidDivisorMask = VectorCompareLE(VectorAbs(Y), GlobalVectorConstants::DoubleSmallNumber);
	
	AlignedDouble4 XDoubles(X), YDoubles(Y);
	XDoubles[0] = fmod(XDoubles[0], YDoubles[0]);
	XDoubles[1] = fmod(XDoubles[1], YDoubles[1]);
	XDoubles[2] = fmod(XDoubles[2], YDoubles[2]);
	XDoubles[3] = fmod(XDoubles[3], YDoubles[3]);
	VectorRegister4Double DoubleResult = XDoubles.ToVectorRegister();

	// Return 0 where divisor Y was too small
	DoubleResult = VectorSelect(InvalidDivisorMask, GlobalVectorConstants::DoubleZero, DoubleResult);
	return DoubleResult;
}

FORCEINLINE VectorRegister4Float VectorSign(VectorRegister4Float X)
{
	VectorRegister4Float Mask = VectorCompareGE(X, GlobalVectorConstants::FloatZero);
	return VectorSelect(Mask, GlobalVectorConstants::FloatOne, GlobalVectorConstants::FloatMinusOne);
}

FORCEINLINE VectorRegister4Double VectorSign(VectorRegister4Double X)
{
	VectorRegister4Double Mask = VectorCompareGE(X, GlobalVectorConstants::DoubleZero);
	return VectorSelect(Mask, GlobalVectorConstants::DoubleOne, GlobalVectorConstants::DoubleMinusOne);
}

FORCEINLINE VectorRegister4Float VectorStep(VectorRegister4Float X)
{
	VectorRegister4Float Mask = VectorCompareGE(X, GlobalVectorConstants::FloatZero);
	return VectorSelect(Mask, GlobalVectorConstants::FloatOne, GlobalVectorConstants::FloatZero);
}

FORCEINLINE VectorRegister4Double VectorStep(VectorRegister4Double X)
{
	VectorRegister4Double Mask = VectorCompareGE(X, GlobalVectorConstants::DoubleZero);
	return VectorSelect(Mask, GlobalVectorConstants::DoubleOne, GlobalVectorConstants::DoubleZero);
}

namespace VectorSinConstantsNEON
{
	static const float p = 0.225f;
	static const float a = 7.58946609f; // 16 * sqrtf(p)
	static const float b = 1.63384342f; // (1 - p) / sqrtf(p)
	static const VectorRegister4Float A = MakeVectorRegisterConstant(a, a, a, a);
	static const VectorRegister4Float B = MakeVectorRegisterConstant(b, b, b, b);
}

FORCEINLINE VectorRegister4Float VectorSin(VectorRegister4Float X)
{
	//Sine approximation using a squared parabola restrained to f(0) = 0, f(PI) = 0, f(PI/2) = 1.
	//based on a good discussion here http://forum.devmaster.net/t/fast-and-accurate-sine-cosine/9648
	//After approx 2.5 million tests comparing to sin(): 
	//Average error of 0.000128
	//Max error of 0.001091
	//
	// Error clarification - the *relative* error rises above 1.2% near
	// 0 and PI (as the result nears 0). This is enough to introduce 
	// harmonic distortion when used as an oscillator - VectorSinCos
	// doesn't cost that much more and is significantly more accurate.
	// (though don't use either for an oscillator if you care about perf)

	VectorRegister4Float Y = VectorMultiply(X, GlobalVectorConstants::OneOverTwoPi);
	Y = VectorSubtract(Y, VectorFloor(VectorAdd(Y, GlobalVectorConstants::FloatOneHalf)));
	Y = VectorMultiply(VectorSinConstantsNEON::A, VectorMultiply(Y, VectorSubtract(GlobalVectorConstants::FloatOneHalf, VectorAbs(Y))));
	return VectorMultiply(Y, VectorAdd(VectorSinConstantsNEON::B, VectorAbs(Y)));
}

FORCEINLINE VectorRegister4Double VectorSin(VectorRegister4Double X)
{
	AlignedDouble4 Doubles(X);
	Doubles[0] = FMath::Sin(Doubles[0]);
	Doubles[1] = FMath::Sin(Doubles[1]);
	Doubles[2] = FMath::Sin(Doubles[2]);
	Doubles[3] = FMath::Sin(Doubles[3]);
	return Doubles.ToVectorRegister();
}

FORCEINLINE VectorRegister4Float VectorCos(VectorRegister4Float X)
{
	return VectorSin(VectorAdd(X, GlobalVectorConstants::PiByTwo));
}

FORCEINLINE VectorRegister4Double VectorCos(VectorRegister4Double X)
{
	AlignedDouble4 Doubles(X);
	Doubles[0] = FMath::Cos(Doubles[0]);
	Doubles[1] = FMath::Cos(Doubles[1]);
	Doubles[2] = FMath::Cos(Doubles[2]);
	Doubles[3] = FMath::Cos(Doubles[3]);
	return Doubles.ToVectorRegister();
}

FORCEINLINE void VectorSinCos(VectorRegister4Double* RESTRICT VSinAngles, VectorRegister4Double* RESTRICT VCosAngles, const VectorRegister4Double* RESTRICT VAngles)
{
	*VSinAngles = VectorSin(*VAngles);
	*VCosAngles = VectorCos(*VAngles);
}

/**
* Loads packed RGBA16(8 bytes) from unaligned memory and converts them into 4 FLOATs.
*
* @param Ptr			Unaligned memory pointer to the RGBA16(8 bytes).
* @return				VectorRegister4Float with 4 FLOATs loaded from Ptr.
*/
FORCEINLINE VectorRegister4Float VectorLoadURGBA16N(const uint16* E)
{
	uint16x4_t UInt16s = vld1_u16(E);
	uint32x4_t UInt32s = vmovl_u16(UInt16s);
	return vcvtq_f32_u32(UInt32s);
}

/**
* Loads packed signed RGBA16(8 bytes) from unaligned memory and converts them into 4 FLOATs.
*
* @param Ptr			Unaligned memory pointer to the RGBA16(8 bytes).
* @return				VectorRegister4Float with 4 FLOATs loaded from Ptr.
*/
FORCEINLINE VectorRegister4Float VectorLoadSRGBA16N(const void* Ptr)
{
	int16x4_t Int16s = vld1_s16((const int16 *)Ptr);
	int32x4_t Int32s = vmovl_s16(Int16s);
	return vcvtq_f32_s32(Int32s);
}

/**
* Converts the 4 FLOATs in the vector RGBA16, clamped to [0, 65535], and stores to unaligned memory.
*
* @param Vec			Vector containing 4 FLOATs
* @param Ptr			Unaligned memory pointer to store the packed RGBA16(8 bytes).
*/
FORCEINLINE void VectorStoreURGBA16N(VectorRegister4Float Vec, uint16* Out)
{
	VectorRegister4Float Tmp;
	Tmp = VectorMax(Vec, VectorZeroFloat());
	Tmp = VectorMin(Tmp, VectorOneFloat());
	Tmp = VectorMultiply(Tmp, vdupq_n_f32(65535.0f));

	uint32x4_t TmpUInt = vcvtnq_u32_f32(Tmp);
	vst1_u16(Out, vmovn_u32(TmpUInt));
}

//////////////////////////////////////////////////////////////////////////
//Integer ops

//Bitwise
/** = a & b */
#define VectorIntAnd(A, B)		vandq_s32(A, B)
/** = a | b */
#define VectorIntOr(A, B)		vorrq_s32(A, B)
/** = a ^ b */
#define VectorIntXor(A, B)		veorq_s32(A, B)
/** = (~a) & b to match _mm_andnot_si128 */
#define VectorIntAndNot(A, B)	vbicq_s32(B, A)
/** = ~a */
#define VectorIntNot(A)	vmvnq_s32(A)

//Comparison
#define VectorIntCompareEQ(A, B)	vceqq_s32(A,B)
#define VectorIntCompareNEQ(A, B)	VectorIntNot(VectorIntCompareEQ(A,B))
#define VectorIntCompareGT(A, B)	vcgtq_s32(A,B)
#define VectorIntCompareLT(A, B)	vcltq_s32(A,B)
#define VectorIntCompareGE(A, B)	vcgeq_s32(A,B)
#define VectorIntCompareLE(A, B)	vcleq_s32(A,B)


FORCEINLINE VectorRegister4Int VectorIntSelect(VectorRegister4Int Mask, VectorRegister4Int Vec1, VectorRegister4Int Vec2)
{
	return vbslq_s32(Mask, Vec1, Vec2);
}

//Arithmetic
#define VectorIntAdd(A, B)	vaddq_s32(A, B)
#define VectorIntSubtract(A, B)	vsubq_s32(A, B)
#define VectorIntMultiply(A, B) vmulq_s32(A, B)
#define VectorIntNegate(A) vnegq_s32(A)
#define VectorIntMin(A, B) vminq_s32(A,B)
#define VectorIntMax(A, B) vmaxq_s32(A,B)
#define VectorIntClamp(A, B, C) VectorIntMin(VectorIntMax(A, B), C)
#define VectorIntAbs(A) vabsq_s32(A)

#define VectorIntSign(A) VectorIntSelect( VectorIntCompareGE(A, GlobalVectorConstants::IntZero), GlobalVectorConstants::IntOne, GlobalVectorConstants::IntMinusOne )

#define VectorIntToFloat(A) vcvtq_f32_s32(A)

FORCEINLINE VectorRegister4Int VectorFloatToInt(VectorRegister4Float A)
{
	return vcvtq_s32_f32(A);
}

FORCEINLINE VectorRegister4Int VectorFloatToInt(VectorRegister4Double A)
{
	return VectorFloatToInt(MakeVectorRegisterFloatFromDouble(A));
}

FORCEINLINE VectorRegister4Int VectorDoubleToInt(VectorRegister4Double Vec)
{
	VectorRegister2Int64 A = vcvtq_s64_f64(Vec.XY);
	VectorRegister2Int64 B = vcvtq_s64_f64(Vec.ZW);
	
	return vcombine_s32(vqmovn_s64(A), vqmovn_s64(B));
}

FORCEINLINE VectorRegister4Int VectorShuffleByte4(VectorRegister4Int Vec, VectorRegister4Int Mask)
{
	return vqtbl1q_u8(Vec, Mask);
}

//Loads and stores

/**
* Stores a vector to memory (aligned or unaligned).
*
* @param Vec	Vector to store
* @param Ptr	Memory pointer
*/
#define VectorIntStore( Vec, Ptr )			vst1q_s32( (int32*)(Ptr), Vec )
#define VectorIntStore_16( Vec, Ptr )		vst1q_s16( (int16*)(Ptr), Vec )

/**
* Loads 4 int32s from unaligned memory.
*
* @param Ptr	Unaligned memory pointer to the 4 int32s
* @return		VectorRegister4Int(Ptr[0], Ptr[1], Ptr[2], Ptr[3])
*/
#define VectorIntLoad( Ptr )				vld1q_s32( (int32*)((void*)(Ptr)) )
#define VectorIntLoad_16( Ptr )				vld1q_s16( (int16*)((void*)(Ptr)) )

/**
* Stores a vector to memory (aligned).
*
* @param Vec	Vector to store
* @param Ptr	Aligned Memory pointer
*/
#define VectorIntStoreAligned( Vec, Ptr )			vst1q_s32( (int32*)(Ptr), Vec )

/**
* Loads 4 int32s from aligned memory.
*
* @param Ptr	Aligned memory pointer to the 4 int32s
* @return		VectorRegister4Int(Ptr[0], Ptr[1], Ptr[2], Ptr[3])
*/
#define VectorIntLoadAligned( Ptr )				vld1q_s32( (int32*)((void*)(Ptr)) )

/**
* Loads 1 int32 from unaligned memory into all components of a vector register.
*
* @param Ptr	Unaligned memory pointer to the 4 int32s
* @return		VectorRegister4Int(*Ptr, *Ptr, *Ptr, *Ptr)
*/
#define VectorIntLoad1(Ptr)							vld1q_dup_s32((int32*)(Ptr))
#define VectorIntLoad1_16(Ptr)                      vld1q_dup_s16((int16*)(Ptr))

#define VectorIntSet1(F)                            (VectorRegister4Int)vdupq_n_s32(F)
#define VectorSetZero()                             vdupq_n_s32(0)
#define VectorSet1(F)                               (VectorRegister4Float)vdupq_n_f32(F)
#define VectorCastIntToFloat(Vec)                   ((VectorRegister4f)vreinterpretq_f32_s32(Vec))
#define VectorCastFloatToInt(Vec)					((VectorRegister4i)vreinterpretq_s32_f32(Vec))
#define VectorCastDoubleToInt(Vec)                  ((VectorRegister4i)vreinterpretq_s64_f64(Vec))
#define VectorCastIntToDouble(Vec)                  ((VectorRegister2Double)vreinterpretq_f64_s64(Vec))
#define VectorShiftLeftImm(Vec, ImmAmt)             vshlq_n_s32(Vec, ImmAmt)
#define VectorShiftRightImmArithmetic(Vec, ImmAmt)  vshrq_n_s32(Vec, ImmAmt)
#define VectorShiftRightImmLogical(Vec, ImmAmt)     vshrq_n_u32(Vec, ImmAmt)
#define VectorRound(Vec)							vrndnq_f32(Vec)

FORCEINLINE VectorRegister4Int VectorRoundToIntHalfToEven(VectorRegister4Float Vec)
{
	return vcvtnq_s32_f32(Vec);
}

FORCEINLINE VectorRegister4i VectorIntExpandLow16To32(VectorRegister4i V)
{
	return vmovl_u16(vget_low_u16(V));
}

// To be continued...

#endif // #if PLATFORM_ENABLE_VECTORINTRINSICS_NEON

PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS
