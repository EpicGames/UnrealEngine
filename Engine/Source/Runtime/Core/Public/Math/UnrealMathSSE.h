// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HAL/PlatformMath.h"
#include "Math/MathFwd.h"
#include "Math/UnrealMathUtility.h"

// IWYU pragma: begin_exports

#if !PLATFORM_ENABLE_VECTORINTRINSICS_NEON && !(defined(__cplusplus_cli)) && PLATFORM_ENABLE_VECTORINTRINSICS

struct VectorRegisterConstInit {};

// We require SSE2
#include <emmintrin.h>

#ifndef UE_PLATFORM_MATH_USE_SSE4_1
#define UE_PLATFORM_MATH_USE_SSE4_1			PLATFORM_ALWAYS_HAS_SSE4_1
#endif

#ifndef UE_PLATFORM_MATH_USE_AVX
#define UE_PLATFORM_MATH_USE_AVX			PLATFORM_ALWAYS_HAS_AVX
#endif

#ifndef UE_PLATFORM_MATH_USE_AVX_2
#define UE_PLATFORM_MATH_USE_AVX_2			(PLATFORM_ALWAYS_HAS_AVX_2 && UE_PLATFORM_MATH_USE_AVX)
#endif

#ifndef UE_PLATFORM_MATH_USE_FMA3
#define UE_PLATFORM_MATH_USE_FMA3			PLATFORM_ALWAYS_HAS_FMA3
#endif

#ifndef UE_PLATFORM_MATH_USE_SVML
	#if defined(_MSC_VER) && !defined(__clang__)
		#define UE_PLATFORM_MATH_USE_SVML			(_MSC_VER >= 1920) // Support added to MSVC 2019 16.0+
	#else
		#define UE_PLATFORM_MATH_USE_SVML			0
	#endif // defined(_MSC_VER)
#endif

#ifndef UE_PLATFORM_MATH_USE_SVML_AVX
#define UE_PLATFORM_MATH_USE_SVML_AVX		(UE_PLATFORM_MATH_USE_SVML && UE_PLATFORM_MATH_USE_AVX)
#endif

// If SSE4.1 is enabled, need additional defines.
#if UE_PLATFORM_MATH_USE_SSE4_1
#include <smmintrin.h>
#endif

// If AVX is enabled, need additional defines.
#if UE_PLATFORM_MATH_USE_AVX || UE_PLATFORM_MATH_USE_SVML
#include <immintrin.h>
#endif

#define UE_SSE_FLOAT_ALIGNMENT	16

#if UE_PLATFORM_MATH_USE_AVX
#define UE_SSE_DOUBLE_ALIGNMENT 32 // required for __m256d
#else
#define UE_SSE_DOUBLE_ALIGNMENT 16
#endif

// We suppress static analysis warnings for the cast from (double*) to (float*) in VectorLoadFloat2
// and VectorLoadTwoPairsFloat below:
// -V:VectorLoadFloat2:615
// -V:VectorLoadTwoPairsFloat:615

/*=============================================================================
 *	Helpers:
 *============================================================================*/

/**
 *	float4 vector register type, where the first float (X) is stored in the lowest 32 bits, and so on.
 */

// 4 floats
typedef __m128	VectorRegister4Float;

// 4 int32s
typedef __m128i VectorRegister4Int;

// 2 int64s
typedef __m128i VectorRegister2Int64;

// 2 doubles
typedef __m128d	VectorRegister2Double;

typedef struct
{
	//TODO: alias for AVX2!
	VectorRegister4Float val[4];
} VectorRegister4x4Float;


namespace SSE
{
	//wrapper for sse_mathfun.h and sse_mathfun_extension.h
	CORE_API VectorRegister4Float sin_ps    (VectorRegister4Float x);
	CORE_API VectorRegister4Float cos_ps    (VectorRegister4Float x);
	CORE_API void                 sincos_ps (VectorRegister4Float x, VectorRegister4Float* s, VectorRegister4Float* c);
	CORE_API VectorRegister4Float log_ps    (VectorRegister4Float x);
	CORE_API VectorRegister4Float exp_ps    (VectorRegister4Float x);
	CORE_API VectorRegister4Float tan_ps    (VectorRegister4Float x);
	CORE_API VectorRegister4Float cot_ps    (VectorRegister4Float x);
	CORE_API VectorRegister4Float atan_ps   (VectorRegister4Float x);
	CORE_API VectorRegister4Float atan2_ps  (VectorRegister4Float y, VectorRegister4Float x);
};

// 4 doubles
struct alignas(UE_SSE_DOUBLE_ALIGNMENT) VectorRegister4Double
{
#if !UE_PLATFORM_MATH_USE_AVX
	VectorRegister2Double XY;
	VectorRegister2Double ZW;

	FORCEINLINE VectorRegister2Double GetXY() const { return XY; }
	FORCEINLINE VectorRegister2Double GetZW() const { return ZW; }
#else
	union
	{
		struct
		{
			VectorRegister2Double XY;
			VectorRegister2Double ZW;
		};
		__m256d XYZW;
	};

	// Use in preference when reading XY or ZW to extract values, it's better on MSVC than the generated memory reads.
	FORCEINLINE VectorRegister2Double GetXY() const { return _mm256_extractf128_pd(XYZW, 0); } // { return _mm256_castpd256_pd128(XYZW); } // Possible MSVC compiler bug in optimized bugs when using this cast, but can be more efficient.
	FORCEINLINE VectorRegister2Double GetZW() const { return _mm256_extractf128_pd(XYZW, 1); }
#endif

	FORCEINLINE VectorRegister4Double() = default;

	FORCEINLINE VectorRegister4Double(VectorRegister2Double InXY, VectorRegister2Double InZW)
	{
#if UE_PLATFORM_MATH_USE_AVX
		XYZW = _mm256_setr_m128d(InXY, InZW);
#else
		XY = InXY;
		ZW = InZW;
#endif
	}

	FORCEINLINE constexpr VectorRegister4Double(VectorRegister2Double InXY, VectorRegister2Double InZW, VectorRegisterConstInit)
	: XY(InXY)
	, ZW(InZW)
	{}

	// Construct from a vector of 4 floats
	FORCEINLINE VectorRegister4Double(VectorRegister4Float FloatVector)
	{
#if !UE_PLATFORM_MATH_USE_AVX
		XY = _mm_cvtps_pd(FloatVector);
		ZW = _mm_cvtps_pd(_mm_movehl_ps(FloatVector, FloatVector));
#else
		XYZW = _mm256_cvtps_pd(FloatVector);
#endif
	}

	// Assign from a vector of 4 floats
	FORCEINLINE VectorRegister4Double& operator=(VectorRegister4Float FloatVector)
	{
#if !UE_PLATFORM_MATH_USE_AVX
		XY = _mm_cvtps_pd(FloatVector);
		ZW = _mm_cvtps_pd(_mm_movehl_ps(FloatVector, FloatVector));
#else
		XYZW = _mm256_cvtps_pd(FloatVector);
#endif
		return *this;
	}

#if UE_PLATFORM_MATH_USE_AVX
	// Convenience for things like 'Result = _mm256_add_pd(...)'
	FORCEINLINE VectorRegister4Double(__m256d Register)
	{
		XYZW = Register;
	}

	// Convenience for things like 'Result = _mm256_add_pd(...)'
	FORCEINLINE VectorRegister4Double& operator=(__m256d Register)
	{
		XYZW = Register;
		return *this;
	}

	// Convenience for passing VectorRegister4Double to _mm256_* functions without needing '.XYZW'
	FORCEINLINE operator __m256d() const
	{
		return XYZW;
	}
#endif

};


// Aliases
typedef VectorRegister4Int VectorRegister4i;
typedef VectorRegister4Float VectorRegister4f;
typedef VectorRegister4Double VectorRegister4d;
typedef VectorRegister2Double VectorRegister2d;
#define VectorZeroVectorRegister() VectorZeroDouble()
#define VectorOneVectorRegister() VectorOneDouble()

// Backwards compatibility
typedef VectorRegister4Double VectorRegister4;
typedef VectorRegister4 VectorRegister;
typedef VectorRegister4Int VectorRegisterInt;


// Forward declarations
VectorRegister4Float VectorLoadAligned(const float* Ptr);
VectorRegister4Double VectorLoadAligned(const double* Ptr);
void VectorStoreAligned(VectorRegister4Float Vec, float* Ptr);
void VectorStoreAligned(VectorRegister4Double Vec, double* Dst);


// Helper for conveniently aligning a float array for extraction from VectorRegister4Float
struct alignas(UE_SSE_FLOAT_ALIGNMENT) AlignedFloat4
{
	float V[4];

	FORCEINLINE AlignedFloat4(VectorRegister4Float Vec)
	{
		VectorStoreAligned(Vec, V);
	}

	FORCEINLINE float operator[](int32 Index) const { return V[Index]; }
	FORCEINLINE float& operator[](int32 Index) { return V[Index]; }

	FORCEINLINE VectorRegister4Float ToVectorRegister() const
	{
		return VectorLoadAligned(V);
	}
};


// Helper for conveniently aligning a double array for extraction from VectorRegister4Double
struct alignas(alignof(VectorRegister4Double)) AlignedDouble4
{
	double V[4];

	FORCEINLINE AlignedDouble4(VectorRegister4Double Vec)
	{
		VectorStoreAligned(Vec, V);
	}

	FORCEINLINE double operator[](int32 Index) const { return V[Index]; }
	FORCEINLINE double& operator[](int32 Index)		 { return V[Index]; }

	FORCEINLINE VectorRegister4Double ToVectorRegister() const
	{
		return VectorLoadAligned(V);
	}
};

typedef AlignedDouble4 AlignedRegister4;

#define DECLARE_VECTOR_REGISTER(X, Y, Z, W) MakeVectorRegister(X, Y, Z, W)

/**
 * @param A0	Selects which element (0-3) from 'A' into 1st slot in the result
 * @param A1	Selects which element (0-3) from 'A' into 2nd slot in the result
 * @param B2	Selects which element (0-3) from 'B' into 3rd slot in the result
 * @param B3	Selects which element (0-3) from 'B' into 4th slot in the result
 */
#define SHUFFLEMASK(A0,A1,B2,B3) ( (A0) | ((A1)<<2) | ((B2)<<4) | ((B3)<<6) )

#define SHUFFLEMASK2(A0,A1) ((A0) | ((A1)<<1))


FORCEINLINE VectorRegister2Double MakeVectorRegister2Double(double X, double Y)
{
	return _mm_setr_pd(X, Y);
}

// Bitwise equivalent from two 64-bit values.
FORCEINLINE VectorRegister2Double MakeVectorRegister2DoubleMask(uint64 X, uint64 Y)
{
	union { VectorRegister2Double Vd; __m128i Vi; } Result;
	// Note: this instruction only exists on 64-bit
	Result.Vi = _mm_set_epi64x(Y, X); // intentionally (Y,X), there is no 'setr' version.
	return Result.Vd;
}

/**
 * Returns a bitwise equivalent vector based on 4 DWORDs.
 *
 * @param X		1st uint32 component
 * @param Y		2nd uint32 component
 * @param Z		3rd uint32 component
 * @param W		4th uint32 component
 * @return		Bitwise equivalent vector with 4 floats
 */
FORCEINLINE VectorRegister4Float MakeVectorRegisterFloat(uint32 X, uint32 Y, uint32 Z, uint32 W)
{
 	union { VectorRegister4Float v; VectorRegister4Int i; } Tmp;
	Tmp.i = _mm_setr_epi32( X, Y, Z, W );
 	return Tmp.v;
}

FORCEINLINE VectorRegister4Double MakeVectorRegisterDouble(uint64 X, uint64 Y, uint64 Z, uint64 W)
{
	return VectorRegister4Double(MakeVectorRegister2DoubleMask(X, Y), MakeVectorRegister2DoubleMask(Z, W));
}

FORCEINLINE VectorRegister4Float MakeVectorRegister(uint32 X, uint32 Y, uint32 Z, uint32 W)
{
	return MakeVectorRegisterFloat(X, Y, Z, W);
}

// Nicer aliases
FORCEINLINE VectorRegister4Float MakeVectorRegisterFloatMask(uint32 X, uint32 Y, uint32 Z, uint32 W)
{
	return MakeVectorRegisterFloat(X, Y, Z, W);
}

FORCEINLINE VectorRegister4Double MakeVectorRegisterDoubleMask(uint64 X, uint64 Y, uint64 Z, uint64 W)
{
	return MakeVectorRegisterDouble(X, Y, Z, W);
}

/**
 * Returns a vector based on 4 FLOATs.
 *
 * @param X		1st float component
 * @param Y		2nd float component
 * @param Z		3rd float component
 * @param W		4th float component
 * @return		Vector of the 4 FLOATs
 */
FORCEINLINE VectorRegister4Float MakeVectorRegisterFloat(float X, float Y, float Z, float W)
{
	return _mm_setr_ps( X, Y, Z, W );
}

FORCEINLINE VectorRegister4Double MakeVectorRegisterDouble(double X, double Y, double Z, double W)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_setr_pd(X, Y);
	Result.ZW = _mm_setr_pd(Z, W);
#else
	Result = _mm256_setr_pd(X, Y, Z, W);
#endif
	return Result;
}

FORCEINLINE VectorRegister4Float MakeVectorRegister(float X, float Y, float Z, float W)
{
	return MakeVectorRegisterFloat(X, Y, Z, W);
}

FORCEINLINE VectorRegister4Double MakeVectorRegister(double X, double Y, double Z, double W)
{
	return MakeVectorRegisterDouble(X, Y, Z, W);
}

FORCEINLINE VectorRegister4Double MakeVectorRegisterDouble(VectorRegister2Double XY, VectorRegister2Double ZW)
{
	return VectorRegister4Double(XY, ZW);
}

// Make double register from float register
FORCEINLINE VectorRegister4Double MakeVectorRegisterDouble(VectorRegister4Float From)
{
	return VectorRegister4Double(From);
}

// Lossy conversion: double->float vector
FORCEINLINE VectorRegister4Float MakeVectorRegisterFloatFromDouble(VectorRegister4Double Vec4d)
{
#if !UE_PLATFORM_MATH_USE_AVX
	return _mm_movelh_ps(_mm_cvtpd_ps(Vec4d.XY), _mm_cvtpd_ps(Vec4d.ZW));
#else
	return _mm256_cvtpd_ps(Vec4d);
#endif
}

/**
* Returns a vector based on 4 int32.
*
* @param X		1st int32 component
* @param Y		2nd int32 component
* @param Z		3rd int32 component
* @param W		4th int32 component
* @return		Vector of the 4 int32
*/
FORCEINLINE VectorRegister4Int MakeVectorRegisterInt(int32 X, int32 Y, int32 Z, int32 W)
{
	return _mm_setr_epi32(X, Y, Z, W);
}

FORCEINLINE VectorRegister2Int64 MakeVectorRegisterInt64(int64 X, int64 Y)
{
	return _mm_set_epi64x(Y, X);
}

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS
#if defined(PRAGMA_DISABLE_MISSING_BRACES_WARNINGS)
PRAGMA_DISABLE_MISSING_BRACES_WARNINGS
#endif

/**
* constexpr 4xint32 vector constant creation that bypasses SIMD intrinsic setter.
*
* Added new function instead of constexprifying MakeVectorRegisterInt to avoid small risk of impacting codegen.
* Long-term we should have a single constexpr MakeVectorRegisterInt.
*/
FORCEINLINE constexpr VectorRegister4Int MakeVectorRegisterIntConstant(int32 X, int32 Y, int32 Z, int32 W)
{
#if !PLATFORM_LITTLE_ENDIAN
#error Big-endian unimplemented
#elif defined(_MSC_VER) && !defined(__clang__)
    return {static_cast<char>(X >> 0), static_cast<char>(X >> 8), static_cast<char>(X >> 16), static_cast<char>(X >> 24),
            static_cast<char>(Y >> 0), static_cast<char>(Y >> 8), static_cast<char>(Y >> 16), static_cast<char>(Y >> 24), 
            static_cast<char>(Z >> 0), static_cast<char>(Z >> 8), static_cast<char>(Z >> 16), static_cast<char>(Z >> 24), 
            static_cast<char>(W >> 0), static_cast<char>(W >> 8), static_cast<char>(W >> 16), static_cast<char>(W >> 24)};
#else
	uint64 XY = uint64(uint32(X)) | (uint64(uint32(Y)) << 32);
	uint64 ZW = uint64(uint32(Z)) | (uint64(uint32(W)) << 32);
    return VectorRegister4Int { (long long)XY, (long long)ZW };
#endif
}

FORCEINLINE constexpr VectorRegister4Float MakeVectorRegisterFloatConstant(float X, float Y, float Z, float W)
{
	return VectorRegister4Float { X, Y, Z, W };
}

#if defined(PRAGMA_ENABLE_MISSING_BRACES_WARNINGS)
PRAGMA_ENABLE_MISSING_BRACES_WARNINGS
#endif
PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS

FORCEINLINE constexpr VectorRegister2Double MakeVectorRegister2DoubleConstant(double X, double Y)
{
	return VectorRegister2Double { X, Y };
}

/*=============================================================================
 *	Constants:
 *============================================================================*/

#include "Math/UnrealMathVectorConstants.h.inl"

/*=============================================================================
 *	Intrinsics:
 *============================================================================*/

/**
 * Returns a vector with all zeros.
 *
 * @return		MakeVectorRegister(0.0f, 0.0f, 0.0f, 0.0f)
 */
FORCEINLINE VectorRegister4Float VectorZeroFloat(void)
{
	return _mm_setzero_ps();
}

FORCEINLINE VectorRegister4Double VectorZeroDouble(void)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_setzero_pd();
	Result.ZW = _mm_setzero_pd();
#else
	Result = _mm256_setzero_pd();
#endif
	return Result;
}

/**
 * Returns a vector with all ones.
 *
 * @return		MakeVectorRegister(1.0f, 1.0f, 1.0f, 1.0f)
 */
FORCEINLINE VectorRegister4Float VectorOneFloat(void)
{
	return GlobalVectorConstants::FloatOne;
}

FORCEINLINE VectorRegister4Double VectorOneDouble(void)
{
	return GlobalVectorConstants::DoubleOne;
}

/**
 * Returns an component from a vector.
 *
 * @param Vec				Vector register
 * @param ComponentIndex	Which component to get, X=0, Y=1, Z=2, W=3
 * @return					The component as a float
 */

template <uint32 ComponentIndex>
FORCEINLINE float VectorGetComponentImpl(VectorRegister4Float Vec)
{
	return (((float*)&(Vec))[ComponentIndex]);
}

// Specializations
template <> FORCEINLINE float VectorGetComponentImpl<0>(VectorRegister4Float Vec)
{
	return _mm_cvtss_f32(Vec); 
}

template <uint32 ComponentIndex>
FORCEINLINE double VectorGetComponentImpl(VectorRegister4Double Vec)
{
#if !UE_PLATFORM_MATH_USE_AVX
	return (((double*)&(Vec.XY))[ComponentIndex]);
#else
	return (((double*)&(Vec.XYZW))[ComponentIndex]);
#endif
}

// Specializations
#if UE_PLATFORM_MATH_USE_AVX
// Lower latency than `vmovsd`, required since MSVC doesn't optimize the above impl well compared to clang/gcc. The latter basically generates something like this below (checked in godbolt).
template <> FORCEINLINE double VectorGetComponentImpl<0>(VectorRegister4Double Vec) { return _mm256_cvtsd_f64(Vec); }
template <> FORCEINLINE double VectorGetComponentImpl<1>(VectorRegister4Double Vec) { return _mm256_cvtsd_f64(_mm256_permute_pd(Vec, 1)); }
template <> FORCEINLINE double VectorGetComponentImpl<2>(VectorRegister4Double Vec) { return _mm_cvtsd_f64(_mm256_extractf128_pd(Vec, 1)); }
template <> FORCEINLINE double VectorGetComponentImpl<3>(VectorRegister4Double Vec) { return _mm_cvtsd_f64(_mm_permute_pd(_mm256_extractf128_pd(Vec, 1), 1)); }
#endif

#define VectorGetComponent(Vec, ComponentIndex) VectorGetComponentImpl<ComponentIndex>(Vec)

FORCEINLINE float VectorGetComponentDynamic(VectorRegister4Float Vec, uint32 ComponentIndex)
{
	return (((float*)&(Vec))[ComponentIndex]);
}

FORCEINLINE double VectorGetComponentDynamic(VectorRegister4Double Vec, uint32 ComponentIndex)
{
#if !UE_PLATFORM_MATH_USE_AVX
	return (((double*)&(Vec.XY))[ComponentIndex]);
#else
	return (((double*)&(Vec.XYZW))[ComponentIndex]);
#endif
}

/**
 * Loads 4 FLOATs from unaligned memory.
 *
 * @param Ptr	Unaligned memory pointer to the 4 FLOATs
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[2], Ptr[3])
 */

FORCEINLINE VectorRegister4Float VectorLoad(const float* Ptr)
{
	return _mm_loadu_ps((float*)(Ptr));
}

FORCEINLINE VectorRegister4Double VectorLoad(const double* Ptr)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_loadu_pd((double*)(Ptr));
	Result.ZW = _mm_loadu_pd((double*)(Ptr + 2));
#else
	Result = _mm256_loadu_pd((double*)Ptr);
#endif
	return Result;
}

/**
 * Loads 16 floats from unaligned memory into 4 vector registers.
 *
 * @param Ptr	Unaligned memory pointer to the 4 floats
 * @return		VectorRegister4x4Float containing 16 floats
 */
FORCEINLINE VectorRegister4x4Float VectorLoad16(const float* Ptr)
{
	VectorRegister4x4Float Result;
	Result.val[0] = VectorLoad(Ptr);
	Result.val[1] = VectorLoad(Ptr + 4);
	Result.val[2] = VectorLoad(Ptr + 8);
	Result.val[3] = VectorLoad(Ptr + 12);
	return Result;
}

/**
 * Loads 3 FLOATs from unaligned memory and sets W=0.
 *
 * @param Ptr	Unaligned memory pointer to the 3 FLOATs
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[2], 0)
 */
FORCEINLINE VectorRegister4Double VectorLoadFloat3(const double* Ptr)
{
#if !UE_PLATFORM_MATH_USE_AVX_2
	VectorRegister4Double Result;
	Result.XY = _mm_loadu_pd((double*)(Ptr));
	Result.ZW = _mm_load_sd((double*)(Ptr+2));
	return Result;
#else
	return _mm256_maskload_pd(Ptr, _mm256_castpd_si256(GlobalVectorConstants::DoubleXYZMask()));
#endif	
}

/**
 * Loads 3 FLOATs from unaligned memory and sets W=1.
 *
 * @param Ptr	Unaligned memory pointer to the 3 FLOATs
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[2], 1.0f)
 */
FORCEINLINE VectorRegister4Double VectorLoadFloat3_W1(const double* Ptr)
{
#if !UE_PLATFORM_MATH_USE_AVX_2
	VectorRegister4Double Result;
	Result.XY = _mm_loadu_pd((double*)(Ptr));
	Result.ZW = MakeVectorRegister2Double(Ptr[2], 1.0);
	return Result;
#else
	//return MakeVectorRegisterDouble(Ptr[0], Ptr[1], Ptr[2], 1.0);
	VectorRegister4Double Result;
	Result = _mm256_maskload_pd(Ptr, _mm256_castpd_si256(GlobalVectorConstants::DoubleXYZMask()));
	Result = _mm256_blend_pd(Result, VectorOneDouble(), 0b1000);
	return Result;
#endif	
}

/**
 * Loads 4 FLOATs from aligned memory.
 *
 * @param Ptr	Aligned memory pointer to the 4 FLOATs
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[2], Ptr[3])
 */
FORCEINLINE VectorRegister4Float VectorLoadAligned(const float* Ptr)
{
	return _mm_load_ps((const float*)(Ptr));
}

FORCEINLINE VectorRegister4Double VectorLoadAligned(const double* Ptr)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_load_pd((const double*)(Ptr));
	Result.ZW = _mm_load_pd((const double*)(Ptr + 2));
#else
	// AVX using unaligned here, since we don't ensure 32-byte alignment (not significant on most modern processors)
	Result = _mm256_loadu_pd(Ptr);
#endif
	return Result;
}

/**
 * Loads 1 float from unaligned memory and replicates it to all 4 elements.
 *
 * @param Ptr	Unaligned memory pointer to the float
 * @return		VectorRegister4Float(Ptr[0], Ptr[0], Ptr[0], Ptr[0])
 */
FORCEINLINE VectorRegister4Float VectorLoadFloat1(const float* Ptr)
{
#if !UE_PLATFORM_MATH_USE_AVX
	return _mm_load1_ps(Ptr);
#else
	return _mm_broadcast_ss(Ptr);
#endif
}

FORCEINLINE VectorRegister4Double VectorLoadDouble1(const double* Ptr)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_load1_pd(Ptr);
	Result.ZW = Result.XY;
#else
	Result = _mm256_broadcast_sd(Ptr);
#endif
	return Result;
}

FORCEINLINE VectorRegister4i VectorLoad64Bits(const void* Ptr)
{
	return _mm_loadu_si64((__m128i *)Ptr);
}

/**
 * Loads 2 floats from unaligned memory into X and Y and duplicates them in Z and W.
 *
 * @param Ptr	Unaligned memory pointer to the floats
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[0], Ptr[1])
 */
FORCEINLINE VectorRegister4Float VectorLoadFloat2(const float* Ptr)
{
	// Switched from _mm_load1_pd and a cast to avoid a compiler bug in VC. This has the benefit of
	// being very clear about not needing any alignment, and the optimizer will still result in
	// movsd and movlhps in both clang and vc.
	return _mm_setr_ps(Ptr[0], Ptr[1], Ptr[0], Ptr[1]);
}

FORCEINLINE VectorRegister4Double VectorLoadFloat2(const double* Ptr)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_loadu_pd(Ptr);
	Result.ZW = Result.XY;
#else
	const __m128d Temp = _mm_loadu_pd(Ptr);
	Result = _mm256_set_m128d(Temp, Temp);
#endif
	return Result;
}

/** 
 * Loads 4 unaligned floats - 2 from the first pointer, 2 from the second, and packs
 * them in to 1 vector.
 *
 * @param Ptr1	Unaligned memory pointer to the first 2 floats
 * @param Ptr2	Unaligned memory pointer to the second 2 floats
 * @return		VectorRegister(Ptr1[0], Ptr1[1], Ptr2[0], Ptr2[1])
 */
FORCEINLINE VectorRegister4Float VectorLoadTwoPairsFloat(const float* Ptr1, const float* Ptr2)
{
	// This intentionally casts to a double* to be able to load 64 bits of data using the "load 1 double" instruction to fill in the two 32-bit floats.
	__m128 Ret = _mm_castpd_ps(_mm_load_sd(reinterpret_cast<const double*>(Ptr1))); // -V615
	Ret = _mm_loadh_pi(Ret, (__m64 const*)(Ptr2));
	return Ret;
}

FORCEINLINE VectorRegister4Double VectorLoadTwoPairsFloat(const double* Ptr1, const double* Ptr2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_loadu_pd(Ptr1);
	Result.ZW = _mm_loadu_pd(Ptr2);
#else
	Result = _mm256_loadu2_m128d(Ptr2, Ptr1); // Note: arguments are (hi, lo)
#endif
	return Result;
}

/**
 * Propagates passed in float to all registers
 *
 * @param F		Float to set
 * @return		VectorRegister4Float(F,F,F,F)
 */
FORCEINLINE VectorRegister4Float VectorSetFloat1(float F)
{
	return _mm_set1_ps(F);
}

FORCEINLINE VectorRegister4Double VectorSetFloat1(double D)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_set1_pd(D);
	Result.ZW = Result.XY;
#else
	Result = _mm256_set1_pd(D);
#endif
	return Result;
}

/**
 * Stores a vector to memory (aligned or unaligned).
 *
 * @param Vec	Vector to store
 * @param Ptr	Memory pointer
 */
FORCEINLINE void VectorStore(VectorRegister4Float Vec, float* Ptr)
{
	_mm_storeu_ps(Ptr, Vec);
}

FORCEINLINE void VectorStore(VectorRegister4Double Vec, double* Dst)
{
#if !UE_PLATFORM_MATH_USE_AVX
	_mm_storeu_pd(Dst, Vec.XY);
	_mm_storeu_pd(Dst + 2, Vec.ZW);
#else
	_mm256_storeu_pd(Dst, Vec);
#endif
}

/**
 * Stores 4 vectors to memory (aligned or unaligned).
 *
 * @param Vec	Vector to store
 * @param Ptr	Memory pointer
 */
FORCEINLINE void VectorStore16(VectorRegister4x4Float Vec, float* Ptr)
{
	VectorStore(Vec.val[0], Ptr);
	VectorStore(Vec.val[1], Ptr + 4);
	VectorStore(Vec.val[2], Ptr + 8);
	VectorStore(Vec.val[3], Ptr + 12);
}

/**
 * Stores a vector to aligned memory.
 *
 * @param Vec	Vector to store
 * @param Ptr	Aligned memory pointer
 */
FORCEINLINE void VectorStoreAligned(VectorRegister4Float Vec, float* Dst)
{
	_mm_store_ps(Dst, Vec);
}

FORCEINLINE void VectorStoreAligned(VectorRegister4Double Vec, double* Dst)
{
#if !UE_PLATFORM_MATH_USE_AVX
	_mm_store_pd(Dst, Vec.XY);
	_mm_store_pd(Dst + 2, Vec.ZW);
#else
	// AVX using unaligned here, since we don't ensure 32-byte alignment (not significant on most modern processors)
	_mm256_storeu_pd(Dst, Vec);
#endif
}


/**
 * Performs non-temporal store of a vector to aligned memory without polluting the caches
 *
 * @param Vec	Vector to store
 * @param Ptr	Aligned memory pointer
 */
FORCEINLINE void VectorStoreAlignedStreamed(VectorRegister4Float Vec, float* Dst)
{
	_mm_stream_ps(Dst, Vec);
}

FORCEINLINE void VectorStoreAlignedStreamed(VectorRegister4Double Vec, double* Dst)
{
#if !UE_PLATFORM_MATH_USE_AVX
	_mm_stream_pd(Dst, Vec.XY);
	_mm_stream_pd(Dst + 2, Vec.ZW);
#else
	// AVX using two 128-bit stores since we don't require 32-byte alignment requirement for our data (so don't use _mm256_stream_pd)
	_mm_stream_pd(Dst, Vec.XY);
	_mm_stream_pd(Dst + 2, Vec.ZW);
#endif
}

/**
 * Stores the XYZ components of a vector to unaligned memory.
 *
 * @param Vec	Vector to store XYZ
 * @param Ptr	Unaligned memory pointer
 */
FORCEINLINE void VectorStoreFloat3(VectorRegister4Float Vec, float* Ptr)
{
	VectorRegister4Float Tmp = Vec;
	_mm_storel_pi((__m64*)(Ptr), Tmp);
	_mm_store_ss(&Ptr[2], _mm_movehl_ps(Tmp, Tmp));
}

FORCEINLINE void VectorStoreFloat3(VectorRegister4Double Vec, double* Dst)
{
#if !UE_PLATFORM_MATH_USE_AVX
	_mm_storeu_pd(Dst, Vec.XY);
	_mm_store_sd(Dst + 2, Vec.ZW);
#else
	_mm_storeu_pd(Dst, Vec.XY);
	_mm_store_sd(Dst + 2, Vec.ZW);
#endif
}

/**
 * Stores the X component of a vector to unaligned memory.
 *
 * @param Vec	Vector to store X
 * @param Ptr	Unaligned memory pointer
 */
FORCEINLINE void VectorStoreFloat1(VectorRegister4Float Vec, float* Ptr)
{
	_mm_store_ss(Ptr, Vec);
}

FORCEINLINE void VectorStoreFloat1(VectorRegister4Double Vec, double* Dst)
{
	_mm_store_sd(Dst, Vec.XY);
}

namespace SSEPermuteHelpers
{
#define InLane0(Index0, Index1)		((Index0) <= 1 && (Index1) <= 1)
#define InLane1(Index0, Index1)		((Index0) >= 2 && (Index1) >= 2)
#define InSameLane(Index0, Index1)	(InLane0(Index0, Index1) || InLane1(Index0, Index1))
#define OutOfLane(Index0, Index1)	(!InSameLane(Index0, Index1))

	/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Double swizzle
	/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	// Double Swizzle helpers
	// Templated swizzles required for double shuffles when using __m128d, since we have to break it down in to two separate operations.

	template <int Index0, int Index1>
	FORCEINLINE VectorRegister2Double SelectVectorSwizzle2(VectorRegister4Double Vec)
	{
		if constexpr (Index0 <= 1)
		{
			if constexpr (Index1 <= 1)
			{
				// [0,1]:[0,1]
				return _mm_shuffle_pd(Vec.GetXY(), Vec.GetXY(), SHUFFLEMASK2(Index0, Index1));
			}
			else
			{
				// [0,1]:[2,3]
				return _mm_shuffle_pd(Vec.GetXY(), Vec.GetZW(), SHUFFLEMASK2(Index0, Index1 - 2));
			}
		}
		else
		{
			if constexpr (Index1 <= 1)
			{
				// [2,3]:[0,1]
				return _mm_shuffle_pd(Vec.GetZW(), Vec.GetXY(), SHUFFLEMASK2(Index0 - 2, Index1));
			}
			else
			{
				// [2,3]:[2,3]
				return _mm_shuffle_pd(Vec.GetZW(), Vec.GetZW(), SHUFFLEMASK2(Index0 - 2, Index1 - 2));
			}
		}
	}

	template<> FORCEINLINE VectorRegister2Double SelectVectorSwizzle2<0, 1>(VectorRegister4Double Vec) { return Vec.GetXY(); }
	template<> FORCEINLINE VectorRegister2Double SelectVectorSwizzle2<2, 3>(VectorRegister4Double Vec) { return Vec.GetZW(); }

#if UE_PLATFORM_MATH_USE_SSE4_1
	// blend can run on more ports than shuffle, so are preferable even if latency is claimed to be the same.
	template<> FORCEINLINE VectorRegister2Double SelectVectorSwizzle2<0, 3>(VectorRegister4Double Vec) { return _mm_blend_pd(Vec.GetXY(), Vec.GetZW(), SHUFFLEMASK2(0, 1)); }
	template<> FORCEINLINE VectorRegister2Double SelectVectorSwizzle2<2, 1>(VectorRegister4Double Vec) { return _mm_blend_pd(Vec.GetZW(), Vec.GetXY(), SHUFFLEMASK2(0, 1)); }
#endif // UE_PLATFORM_MATH_USE_SSE4_1


#if UE_PLATFORM_MATH_USE_AVX

	// Helper to swap lanes (128-bit pairs)
	constexpr int PERMUTE_LANE_MASK(int A, int B) { return (A == 0 ? 0x00 : 0x01) | (B == 0 ? (0x02 << 4) : (0x03 << 4)); }

	template<int Lane0, int Lane1>
	FORCEINLINE VectorRegister4Double PermuteLanes(VectorRegister4Double Vec)
	{
		static_assert(Lane0 >= 0 && Lane0 <= 1 && Lane1 >= 0 && Lane1 <= 1, "Invalid Index");
		return _mm256_permute2f128_pd(Vec, Vec, PERMUTE_LANE_MASK(Lane0, Lane1));
	}

	// Identity
	template<> FORCEINLINE VectorRegister4Double PermuteLanes<0, 1>(VectorRegister4Double Vec) { return Vec; }
#if !UE_PLATFORM_MATH_USE_AVX_2
	// On AVX1, permute2f128 can be quite slow, so look for alternatives (extract + insert). On AVX2, permute2f128 is more efficient and should equal or beat (extract + insert).
	// Sources: https://www.agner.org/optimize/instruction_tables.pdf, https://uops.info/table.html
	template<> FORCEINLINE VectorRegister4Double PermuteLanes<0, 0>(VectorRegister4Double Vec) { return _mm256_insertf128_pd(Vec, Vec.GetXY(), 1); } // copy XY to lane 1
	template<> FORCEINLINE VectorRegister4Double PermuteLanes<1, 0>(VectorRegister4Double Vec) { return _mm256_setr_m128d(Vec.GetZW(), Vec.GetXY()); } // swap XY and ZW
	template<> FORCEINLINE VectorRegister4Double PermuteLanes<1, 1>(VectorRegister4Double Vec) { return _mm256_insertf128_pd(Vec, Vec.GetZW(), 0); } // copy ZW to lane 0
#endif // !AVX2

	//
	// AVX2 _mm256_permute4x64_pd has a latency of 3-6, but there are some specializations using instructions which have a latency of 1 but are restricted to in-lane (128 bit) permutes.
	// AVX1 benefits from lower latency instructions here than the toggling between 128-bit and 256-bit operations of the generic implementation.

	constexpr int PERMUTE_MASK(int A, int B, int C, int D) { return ((A == 1 ? (1 << 0) : 0) | (B == 1 ? (1 << 1) : 0) | (C == 3 ? (1 << 2) : 0) | (D == 3 ? (1 << 3) : 0)); }

	template <int Index0, int Index1, int Index2, int Index3>
	FORCEINLINE VectorRegister4Double SelectVectorSwizzle(VectorRegister4Double Vec)
	{
		if constexpr (InLane0(Index0, Index1) && InLane1(Index2, Index3))
		{
			// [0..1][0..1][2..3][2..3]
			return _mm256_permute_pd(Vec, PERMUTE_MASK(Index0, Index1, Index2, Index3));
		}
		else if constexpr (InLane1(Index0, Index1) && InLane0(Index2, Index3))
		{
			// [2..3][2..3][0..1][0..1]
			// Permute lanes then use [lane0][lane1] swizzle
			return SelectVectorSwizzle<Index0 - 2, Index1 - 2, Index2 + 2, Index3 + 2>(PermuteLanes<1, 0>(Vec));
		}
		else if constexpr (InLane0(Index0, Index1) && InLane0(Index2, Index3))
		{
			// [0..1][0..1][0..1][0..1]
			// Permute lanes then use [lane0][lane1] swizzle
			return SelectVectorSwizzle<Index0, Index1, Index2 + 2, Index3 + 2>(PermuteLanes<0, 0>(Vec));
		}
		else if constexpr (InLane1(Index0, Index1) && InLane1(Index2, Index3))
		{
			// [2..3][2..3][2..3][2..3]
			// Permute lanes then use [lane0][lane1] swizzle
			return SelectVectorSwizzle<Index0 - 2, Index1 - 2, Index2, Index3>(PermuteLanes<1, 1>(Vec));
		}
		else
		{
			// Anything with out-of-lane pairs
#if UE_PLATFORM_MATH_USE_AVX_2
			return _mm256_permute4x64_pd(Vec, SHUFFLEMASK(Index0, Index1, Index2, Index3));
#else
			return VectorRegister4Double(
				SelectVectorSwizzle2<Index0, Index1>(Vec),
				SelectVectorSwizzle2<Index2, Index3>(Vec)
			);
#endif
		}
	}

	//
	// Specializations
	//
	template<> FORCEINLINE VectorRegister4Double SelectVectorSwizzle<0, 0, 2, 2>(VectorRegister4Double Vec) { return _mm256_movedup_pd(Vec); } // special instruction exists for this.
	template<> FORCEINLINE VectorRegister4Double SelectVectorSwizzle<0, 1, 2, 3>(VectorRegister4Double Vec) { return Vec; } // Identity
	template<> FORCEINLINE VectorRegister4Double SelectVectorSwizzle<2, 3, 0, 1>(VectorRegister4Double Vec) { return PermuteLanes<1, 0>(Vec); }
	template<> FORCEINLINE VectorRegister4Double SelectVectorSwizzle<0, 1, 0, 1>(VectorRegister4Double Vec) { return PermuteLanes<0, 0>(Vec); }
	template<> FORCEINLINE VectorRegister4Double SelectVectorSwizzle<2, 3, 2, 3>(VectorRegister4Double Vec) { return PermuteLanes<1, 1>(Vec); }

#endif // AVX

	// Double swizzle wrapper
	template<int Index0, int Index1, int Index2, int Index3>
	FORCEINLINE VectorRegister4Double VectorSwizzleTemplate(VectorRegister4Double Vec)
	{
		static_assert(Index0 >= 0 && Index0 <= 3 && Index1 >= 0 && Index1 <= 3 && Index2 >= 0 && Index2 <= 3 && Index3 >= 0 && Index3 <= 3, "Invalid Index");

#if UE_PLATFORM_MATH_USE_AVX
		return SelectVectorSwizzle<Index0, Index1, Index2, Index3>(Vec);
#else
		return VectorRegister4Double(
			SelectVectorSwizzle2<Index0, Index1>(Vec),
			SelectVectorSwizzle2<Index2, Index3>(Vec)
		);
#endif
	}

	// Specializations
	template<> FORCEINLINE VectorRegister4Double VectorSwizzleTemplate<0, 1, 2, 3>(VectorRegister4Double Vec) { return Vec; } // Identity

	/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Double replicate
	/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	template <int Index>
	FORCEINLINE VectorRegister2Double VectorReplicateImpl2(VectorRegister2Double Vec)
	{
		// Note: 2 doubles (VectorRegister2Double / m128d)
		return _mm_shuffle_pd(Vec, Vec, SHUFFLEMASK2(Index, Index));
	}

	// Double replicate (4 doubles)
	template <int Index>
	FORCEINLINE VectorRegister4Double VectorReplicateImpl4(VectorRegister4Double Vec)
	{
		if constexpr (Index <= 1)
		{
			VectorRegister2Double Temp = VectorReplicateImpl2<Index>(Vec.GetXY());
			return VectorRegister4Double(Temp, Temp);
		}
		else
		{
			VectorRegister2Double Temp = VectorReplicateImpl2<Index - 2>(Vec.GetZW());
			return VectorRegister4Double(Temp, Temp);
		}
	}

	//
	// Double replicate wrapper
	//
	template<int Index>
	FORCEINLINE VectorRegister4Double VectorReplicateTemplate(VectorRegister4Double Vec)
	{
		static_assert(Index >= 0 && Index <= 3, "Invalid Index");

#if UE_PLATFORM_MATH_USE_AVX_2
		return VectorSwizzleTemplate<Index, Index, Index, Index>(Vec);
#else
		return VectorReplicateImpl4<Index>(Vec);
#endif
	}

	/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Double shuffle
	/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if UE_PLATFORM_MATH_USE_AVX

	//
	// Lane shuffle helper
	//
	template<int Lane0, int Lane1>
	FORCEINLINE VectorRegister4Double ShuffleLanes(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
	{
		static_assert(Lane0 >= 0 && Lane0 <= 1 && Lane1 >= 0 && Lane1 <= 1, "Invalid Index");
		return _mm256_permute2f128_pd(Vec1, Vec2, PERMUTE_LANE_MASK(Lane0, Lane1));
	}

	// Lane shuffle helper specialization
	template<> FORCEINLINE VectorRegister4Double ShuffleLanes<0, 1>(VectorRegister4Double Vec1, VectorRegister4Double Vec2) { return _mm256_blend_pd(Vec1, Vec2, 0b1100); }
#if !UE_PLATFORM_MATH_USE_AVX_2
	// On AVX1, permute2f128 can be quite slow, so look for alternatives (extract + insert). On AVX2, permute2f128 is more efficient and should equal or beat (extract + insert).
	// Sources: https://www.agner.org/optimize/instruction_tables.pdf, https://uops.info/table.html
	template<> FORCEINLINE VectorRegister4Double ShuffleLanes<0, 0>(VectorRegister4Double Vec1, VectorRegister4Double Vec2) { return _mm256_insertf128_pd(Vec1, Vec2.GetXY(), 1); } // copy XY to lane 1
	template<> FORCEINLINE VectorRegister4Double ShuffleLanes<1, 0>(VectorRegister4Double Vec1, VectorRegister4Double Vec2) { return _mm256_setr_m128d(Vec1.GetZW(), Vec2.GetXY()); } // swap XY and ZW
	template<> FORCEINLINE VectorRegister4Double ShuffleLanes<1, 1>(VectorRegister4Double Vec1, VectorRegister4Double Vec2) { return _mm256_insertf128_pd(Vec2, Vec1.GetZW(), 0); } // copy ZW to lane 0
#endif // !AVX2

	//
	// Double shuffle helpers
	//

	// When index pairs are within the same lane, SelectVectorShuffle first efficiently blends elements from the two vectors,
	// then efficiently swizzles within 128-bit lanes using specializations for indices [0..1][0..1][2..3][2..3]
	// 
	template <int Index0, int Index1, int Index2, int Index3>
	FORCEINLINE VectorRegister4Double SelectVectorShuffle(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
	{
		if constexpr (InLane0(Index0, Index1) && InLane1(Index2, Index3))
		{
			// [0..1][0..1][2..3][2..3]
			const VectorRegister4Double Blended = ShuffleLanes<0, 1>(Vec1, Vec2);
			return VectorSwizzleTemplate<Index0, Index1, Index2, Index3>(Blended);
		}
		else if constexpr (InLane1(Index0, Index1) && InLane0(Index2, Index3))
		{
			// [2..3][2..3][0..1][0..1]
			const VectorRegister4Double Blended = ShuffleLanes<1, 0>(Vec1, Vec2);
			return VectorSwizzleTemplate<Index0 - 2, Index1 - 2, Index2 + 2, Index3 + 2>(Blended);
		}
		else if constexpr (InLane0(Index0, Index1) && InLane0(Index2, Index3))
		{
			// [0..1][0..1][0..1][0..1]
			const VectorRegister4Double Blended = ShuffleLanes<0, 0>(Vec1, Vec2);
			return VectorSwizzleTemplate<Index0, Index1, Index2 + 2, Index3 + 2>(Blended);
		}
		else if constexpr (InLane1(Index0, Index1) && InLane1(Index2, Index3))
		{
			// [2..3][2..3][2..3][2..3]
			const VectorRegister4Double Blended = ShuffleLanes<1, 1>(Vec1, Vec2);
			return VectorSwizzleTemplate<Index0 - 2, Index1 - 2, Index2, Index3>(Blended);
		}
		else if constexpr (InSameLane(Index0, Index1) && OutOfLane(Index2, Index3))
		{
			const VectorRegister4Double Vec1_XY = VectorSwizzleTemplate<Index0, Index1, 2, 3>(Vec1);
			const VectorRegister2Double Vec2_ZW = SelectVectorSwizzle2<Index2, Index3>(Vec2);
			return _mm256_insertf128_pd(Vec1_XY, Vec2_ZW, 0x1);
		}
		else if constexpr (OutOfLane(Index0, Index1) && InSameLane(Index2, Index3))
		{
			const VectorRegister2Double Vec1_XY = SelectVectorSwizzle2<Index0, Index1>(Vec1);
			const VectorRegister4Double Vec2_ZW = VectorSwizzleTemplate<0, 1, Index2, Index3>(Vec2);
			return _mm256_insertf128_pd(Vec2_ZW, Vec1_XY, 0x0);
		}
		else
		{
			return VectorRegister4Double(
				SelectVectorSwizzle2<Index0, Index1>(Vec1),
				SelectVectorSwizzle2<Index2, Index3>(Vec2)
			);
		}
	}

	// AVX Double Shuffle specializations 
	// Shuffles of 128-bit pairs, ie combinations of [0,1][2,3].
	template<> FORCEINLINE VectorRegister4Double SelectVectorShuffle<0, 1, 0, 1>(VectorRegister4Double Vec1, VectorRegister4Double Vec2) { return ShuffleLanes<0, 0>(Vec1, Vec2); }
	template<> FORCEINLINE VectorRegister4Double SelectVectorShuffle<0, 1, 2, 3>(VectorRegister4Double Vec1, VectorRegister4Double Vec2) { return ShuffleLanes<0, 1>(Vec1, Vec2); }
	template<> FORCEINLINE VectorRegister4Double SelectVectorShuffle<2, 3, 0, 1>(VectorRegister4Double Vec1, VectorRegister4Double Vec2) { return ShuffleLanes<1, 0>(Vec1, Vec2); }
	template<> FORCEINLINE VectorRegister4Double SelectVectorShuffle<2, 3, 2, 3>(VectorRegister4Double Vec1, VectorRegister4Double Vec2) { return ShuffleLanes<1, 1>(Vec1, Vec2); }

#else

	// Non-AVX implementation
	template<int Index0, int Index1, int Index2, int Index3>
	FORCEINLINE VectorRegister4Double SelectVectorShuffle(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
	{
		return VectorRegister4Double(
			SelectVectorSwizzle2<Index0, Index1>(Vec1),
			SelectVectorSwizzle2<Index2, Index3>(Vec2)
		);
	}

#endif // AVX

	//
	// Double shuffle wrapper
	//
	template<int Index0, int Index1, int Index2, int Index3>
	FORCEINLINE VectorRegister4Double VectorShuffleTemplate(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
	{
		static_assert(Index0 >= 0 && Index0 <= 3 && Index1 >= 0 && Index1 <= 3 && Index2 >= 0 && Index2 <= 3 && Index3 >= 0 && Index3 <= 3, "Invalid Index");
		return SelectVectorShuffle<Index0, Index1, Index2, Index3>(Vec1, Vec2);
	}

	/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Float swizzle
	/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	template<int Index0, int Index1, int Index2, int Index3>
	FORCEINLINE VectorRegister4Float VectorSwizzleTemplate(VectorRegister4Float Vec)
	{
		VectorRegister4Float Result = _mm_shuffle_ps(Vec, Vec, SHUFFLEMASK(Index0, Index1, Index2, Index3));
		return Result;
	}

	// Float Swizzle specializations.
	// These can result in no-ops or simpler ops than shuffle which don't compete with the shuffle unit, or which can copy directly to the destination and avoid an intermediate mov.
	// See: https://stackoverflow.com/questions/56238197/what-is-the-difference-between-mm-movehdup-ps-and-mm-shuffle-ps-in-this-case
	template<> FORCEINLINE VectorRegister4Float VectorSwizzleTemplate<0, 1, 2, 3>(VectorRegister4Float Vec) { return Vec; }
	template<> FORCEINLINE VectorRegister4Float VectorSwizzleTemplate<0, 1, 0, 1>(VectorRegister4Float Vec) { return _mm_movelh_ps(Vec, Vec); }
	template<> FORCEINLINE VectorRegister4Float VectorSwizzleTemplate<2, 3, 2, 3>(VectorRegister4Float Vec) { return _mm_movehl_ps(Vec, Vec); }
	template<> FORCEINLINE VectorRegister4Float VectorSwizzleTemplate<0, 0, 1, 1>(VectorRegister4Float Vec) { return _mm_unpacklo_ps(Vec, Vec); }
	template<> FORCEINLINE VectorRegister4Float VectorSwizzleTemplate<2, 2, 3, 3>(VectorRegister4Float Vec) { return _mm_unpackhi_ps(Vec, Vec); }

#if UE_PLATFORM_MATH_USE_SSE4_1
	template<> FORCEINLINE VectorRegister4Float VectorSwizzleTemplate<0, 0, 2, 2>(VectorRegister4Float Vec) { return _mm_moveldup_ps(Vec); }
	template<> FORCEINLINE VectorRegister4Float VectorSwizzleTemplate<1, 1, 3, 3>(VectorRegister4Float Vec) { return _mm_movehdup_ps(Vec); }
#endif

#if UE_PLATFORM_MATH_USE_AVX_2
	template<> FORCEINLINE VectorRegister4Float VectorSwizzleTemplate<0, 0, 0, 0>(VectorRegister4Float Vec) { return _mm_broadcastss_ps(Vec); }
#endif

	/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Float replicate
	template<int Index>
	FORCEINLINE VectorRegister4Float VectorReplicateTemplate(VectorRegister4Float Vec)
	{
		static_assert(Index >= 0 && Index <= 3, "Invalid Index");
		return VectorSwizzleTemplate<Index, Index, Index, Index>(Vec);
	}

	/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Float shuffle
	template<int Index0, int Index1, int Index2, int Index3>
	FORCEINLINE VectorRegister4Float VectorShuffleTemplate(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
	{
		static_assert(Index0 >= 0 && Index0 <= 3 && Index1 >= 0 && Index1 <= 3 && Index2 >= 0 && Index2 <= 3 && Index3 >= 0 && Index3 <= 3, "Invalid Index");
		return _mm_shuffle_ps(Vec1, Vec2, SHUFFLEMASK(Index0, Index1, Index2, Index3));
	}

	// Float Shuffle specializations
	template<> FORCEINLINE VectorRegister4Float VectorShuffleTemplate<0, 1, 0, 1>(VectorRegister4Float Vec1, VectorRegister4Float Vec2) { return _mm_movelh_ps(Vec1, Vec2); }
	template<> FORCEINLINE VectorRegister4Float VectorShuffleTemplate<2, 3, 2, 3>(VectorRegister4Float Vec1, VectorRegister4Float Vec2) { return _mm_movehl_ps(Vec2, Vec1); } // Note: movehl copies first from the 2nd argument

#undef OutOfLane
#undef InSameLane
#undef InLane1
#undef InLane0
} // namespace SSEPermuteHelpers

/**
 * Replicates one element into all four elements and returns the new vector.
 *
 * @param Vec			Source vector
 * @param ElementIndex	Index (0-3) of the element to replicate
 * @return				VectorRegister4Float( Vec[ElementIndex], Vec[ElementIndex], Vec[ElementIndex], Vec[ElementIndex] )
 */

#define VectorReplicate(Vec, ElementIndex)	SSEPermuteHelpers::VectorReplicateTemplate<ElementIndex>(Vec)

 /**
  * Swizzles the 4 components of a vector and returns the result.
  *
  * @param Vec		Source vector
  * @param X			Index for which component to use for X (literal 0-3)
  * @param Y			Index for which component to use for Y (literal 0-3)
  * @param Z			Index for which component to use for Z (literal 0-3)
  * @param W			Index for which component to use for W (literal 0-3)
  * @return			The swizzled vector
  */
#define VectorSwizzle(Vec, X, Y, Z, W)		SSEPermuteHelpers::VectorSwizzleTemplate<X,Y,Z,W>(Vec)

  /**
   * Creates a vector through selecting two components from each vector via a shuffle mask.
   *
   * @param Vec1		Source vector1
   * @param Vec2		Source vector2
   * @param X			Index for which component of Vector1 to use for X (literal 0-3)
   * @param Y			Index for which component of Vector1 to use for Y (literal 0-3)
   * @param Z			Index for which component of Vector2 to use for Z (literal 0-3)
   * @param W			Index for which component of Vector2 to use for W (literal 0-3)
   * @return			The swizzled vector
   */
#define VectorShuffle(Vec1, Vec2, X, Y, Z, W)		SSEPermuteHelpers::VectorShuffleTemplate<X,Y,Z,W>(Vec1, Vec2)


/**
 * Returns the absolute value (component-wise).
 *
 * @param Vec			Source vector
 * @return				VectorRegister4Float( abs(Vec.x), abs(Vec.y), abs(Vec.z), abs(Vec.w) )
 */
FORCEINLINE VectorRegister4Float VectorAbs(VectorRegister4Float Vec)
{
	return _mm_and_ps(Vec, GlobalVectorConstants::SignMask());
}

FORCEINLINE VectorRegister4Double VectorAbs(VectorRegister4Double Vec)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	VectorRegister2Double DoubleSignMask2d = MakeVectorRegister2DoubleMask(~(uint64(1) << 63), ~(uint64(1) << 63));
	Result.XY = _mm_and_pd(Vec.XY, DoubleSignMask2d);
	Result.ZW = _mm_and_pd(Vec.ZW, DoubleSignMask2d);
#else
	Result = _mm256_and_pd(Vec, GlobalVectorConstants::DoubleSignMask());
#endif
	return Result;
}

/**
 * Returns the negated value (component-wise).
 *
 * @param Vec			Source vector
 * @return				VectorRegister4Float( -Vec.x, -Vec.y, -Vec.z, -Vec.w )
 */
FORCEINLINE VectorRegister4Float VectorNegate(VectorRegister4Float Vec)
{
	return _mm_sub_ps(_mm_setzero_ps(), Vec);
}

FORCEINLINE VectorRegister4Double VectorNegate(VectorRegister4Double Vec)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_sub_pd(_mm_setzero_pd(), Vec.XY);
	Result.ZW = _mm_sub_pd(_mm_setzero_pd(), Vec.ZW);
#else
	Result = _mm256_sub_pd(_mm256_setzero_pd(), Vec);
#endif
	return Result;
}

/**
 * Adds two vectors (component-wise) and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x+Vec2.x, Vec1.y+Vec2.y, Vec1.z+Vec2.z, Vec1.w+Vec2.w )
 */

FORCEINLINE VectorRegister4Float VectorAdd(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return _mm_add_ps(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorAdd(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_add_pd(Vec1.XY, Vec2.XY);
	Result.ZW = _mm_add_pd(Vec1.ZW, Vec2.ZW);
#else
	Result = _mm256_add_pd(Vec1, Vec2);
#endif
	return Result;
}

/**
 * Subtracts a vector from another (component-wise) and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x-Vec2.x, Vec1.y-Vec2.y, Vec1.z-Vec2.z, Vec1.w-Vec2.w )
 */
FORCEINLINE VectorRegister4Float VectorSubtract(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return _mm_sub_ps(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorSubtract(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_sub_pd(Vec1.XY, Vec2.XY);
	Result.ZW = _mm_sub_pd(Vec1.ZW, Vec2.ZW);
#else
	Result = _mm256_sub_pd(Vec1, Vec2);
#endif
	return Result;
}

/**
 * Multiplies two vectors (component-wise) and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x*Vec2.x, Vec1.y*Vec2.y, Vec1.z*Vec2.z, Vec1.w*Vec2.w )
 */
FORCEINLINE VectorRegister4Float VectorMultiply(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return _mm_mul_ps(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorMultiply(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_mul_pd(Vec1.XY, Vec2.XY);
	Result.ZW = _mm_mul_pd(Vec1.ZW, Vec2.ZW);
#else
	Result = _mm256_mul_pd(Vec1, Vec2);
#endif
	return Result;
}



/**
 * Multiplies two vectors (component-wise), adds in the third vector and returns the result. (A*B + C)
 *
 * @param A		1st vector
 * @param B		2nd vector
 * @param C		3rd vector
 * @return		VectorRegister4Float( A.x*B.x + C.x, A.y*B.y + C.y, A.z*B.z + C.z, A.w*B.w + C.w )
 */
FORCEINLINE VectorRegister4Float VectorMultiplyAdd(VectorRegister4Float A, VectorRegister4Float B, VectorRegister4Float C)
{
#if UE_PLATFORM_MATH_USE_FMA3
	return _mm_fmadd_ps(A, B, C);
#else
	return VectorAdd(VectorMultiply(A, B), C);
#endif
}

FORCEINLINE VectorRegister4Double VectorMultiplyAdd(VectorRegister4Double A, VectorRegister4Double B, VectorRegister4Double C)
{
#if UE_PLATFORM_MATH_USE_FMA3 && UE_PLATFORM_MATH_USE_AVX
	return _mm256_fmadd_pd(A, B, C);
#elif UE_PLATFORM_MATH_USE_FMA3
	VectorRegister4Double Result;
	Result.XY = _mm_fmadd_pd(A.XY, B.XY, C.XY);
	Result.ZW = _mm_fmadd_pd(A.ZW, B.ZW, C.ZW);
	return Result;
#else
	return VectorAdd(VectorMultiply(A, B), C);
#endif
}

/**
 * Multiplies two vectors (component-wise), negates the results and adds it to the third vector i.e. (-A*B + C) = (C - A*B)
 *
 * @param A		1st vector
 * @param B		2nd vector
 * @param C		3rd vector
 * @return		VectorRegister( C.x - A.x*B.x, C.y - A.y*B.y, C.z - A.z*B.z, C.w - A.w*B.w )
 */
FORCEINLINE VectorRegister4Float VectorNegateMultiplyAdd(VectorRegister4Float A, VectorRegister4Float B, VectorRegister4Float C)
{
#if UE_PLATFORM_MATH_USE_FMA3
	return _mm_fnmadd_ps(A, B, C);
#else
	return VectorSubtract(C, VectorMultiply(A, B));
#endif
}

FORCEINLINE VectorRegister4Double VectorNegateMultiplyAdd(VectorRegister4Double A, VectorRegister4Double B, VectorRegister4Double C)
{
#if UE_PLATFORM_MATH_USE_FMA3 && UE_PLATFORM_MATH_USE_AVX
	return _mm256_fnmadd_pd(A, B, C);
#elif UE_PLATFORM_MATH_USE_FMA3
	VectorRegister4Double Result;
	Result.XY = _mm_fnmadd_pd(A.XY, B.XY, C.XY);
	Result.ZW = _mm_fnmadd_pd(A.ZW, B.ZW, C.ZW);
	return Result;
#else
	return VectorSubtract(C, VectorMultiply(A, B));
#endif
}


/**
 * Divides two vectors (component-wise) and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x/Vec2.x, Vec1.y/Vec2.y, Vec1.z/Vec2.z, Vec1.w/Vec2.w )
 */
FORCEINLINE VectorRegister4Float VectorDivide(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return _mm_div_ps(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorDivide(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_div_pd(Vec1.XY, Vec2.XY);
	Result.ZW = _mm_div_pd(Vec1.ZW, Vec2.ZW);
#else
	Result = _mm256_div_pd(Vec1, Vec2);
#endif
	return Result;
}

namespace SSEVectorHelperFuncs
{
	// Computes VectorDot3 but only with the result in the first (X) element of a VectorRegister4Float
	FORCEINLINE VectorRegister4Float InternalVectorDot3X(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
	{
		// (X, Y, Z, W)
		VectorRegister4Float Prod = VectorMultiply(Vec1, Vec2);
		// (Y, Y, W, W)
		VectorRegister4Float Shuf = VectorSwizzle(Prod, 1, 1, 3, 3); // _mm_movehdup_ps on SSE4.1, shuffle otherwise
		// (X+Y, ???, ???, ???)
		VectorRegister4Float Sum = VectorAdd(Prod, Shuf);
		// (Z, W, Z, W)
		Shuf = VectorSwizzle(Prod, 2, 3, 2, 3); // _mm_movehl_ps
		// (X+Y+Z, ???, ???, ???)
		Sum = VectorAdd(Sum, Shuf);
		return Sum;
	}

	// Computes VectorDot3 but only with the result in the first (X) element of a VectorRegister4Double
	FORCEINLINE VectorRegister4Double InternalVectorDot3X_Full(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
	{
		// (X, Y, Z, W)
		VectorRegister4Double Prod = VectorMultiply(Vec1, Vec2);
		// (Y, Y, W, W)
		VectorRegister4Double Shuf = VectorSwizzle(Prod, 1, 1, 3, 3); // fast in-lane permute on AVX (_mm256_permute_pd)
		// (X+Y, ???, ???, ???)
		VectorRegister4Double Sum = VectorAdd(Prod, Shuf);
		// (Z, W, Z, W)
		Shuf = VectorSwizzle(Prod, 2, 3, 2, 3); // various specializations exist for this depending on platform
		// (X+Y+Z, ???, ???, ???)
		Sum = VectorAdd(Sum, Shuf);

		return Sum;
	}

	// Computes VectorDot3 but only with the result in the first (X) element of a VectorRegister2Double (half of VectorRegister4Double)
	FORCEINLINE VectorRegister2Double InternalVectorDot3X_Half(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
	{
		VectorRegister2Double T, A;

		// (X, Y)
		T = _mm_mul_pd(Vec1.XY, Vec2.XY);

		// (X + Z, Y + W)
#if UE_PLATFORM_MATH_USE_FMA3
		A = _mm_fmadd_pd(Vec1.ZW, Vec2.ZW, T);
#else
		A = _mm_add_pd(_mm_mul_pd(Vec1.ZW, Vec2.ZW), T);
#endif // UE_PLATFORM_MATH_USE_FMA3

		// (Y, X)  // Reverse of T
		T = _mm_shuffle_pd(T, T, SHUFFLEMASK2(1, 0));

		// (X + Z + Y, Y + W + X)
		T = _mm_add_pd(A, T);

		return T;
	}

} // namespace SSEVectorHelperFuncs


/**
 * Calculates the dot3 product of two vectors and returns a scalar value.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		d = dot3(Vec1.xyz, Vec2.xyz)
 */
FORCEINLINE float VectorDot3Scalar(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return VectorGetComponent(SSEVectorHelperFuncs::InternalVectorDot3X(Vec1, Vec2), 0);
}

FORCEINLINE double VectorDot3Scalar(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
#if UE_PLATFORM_MATH_USE_AVX
	return VectorGetComponent(SSEVectorHelperFuncs::InternalVectorDot3X_Full(Vec1, Vec2), 0);
#else
	VectorRegister2Double T = SSEVectorHelperFuncs::InternalVectorDot3X_Half(Vec1, Vec2);
	// Extract first component
	return _mm_cvtsd_f64(T);
#endif
}

/**
 * Calculates the dot3 product of two vectors and returns a vector with the result in all 4 components.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		d = dot3(Vec1.xyz, Vec2.xyz), VectorRegister4Float( d, d, d, d )
 */
FORCEINLINE VectorRegister4Float VectorDot3(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return VectorReplicate(SSEVectorHelperFuncs::InternalVectorDot3X(Vec1, Vec2), 0);
}

FORCEINLINE VectorRegister4Double VectorDot3(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
#if UE_PLATFORM_MATH_USE_AVX
	return VectorReplicate(SSEVectorHelperFuncs::InternalVectorDot3X_Full(Vec1, Vec2), 0);
#else
	VectorRegister2Double T = SSEVectorHelperFuncs::InternalVectorDot3X_Half(Vec1, Vec2);
	// Replicate in half (X,X)
	T = _mm_shuffle_pd(T, T, SHUFFLEMASK2(0, 0));
	// Replicate in full (X,X,X,X)
	return VectorRegister4Double(T, T);
#endif	
}

/**
 * Calculates the dot4 product of two vectors and returns a vector with the result in all 4 components.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		d = dot4(Vec1, Vec2), VectorRegister4Float( d, d, d, d )
 */
FORCEINLINE VectorRegister4Float VectorDot4(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	VectorRegister4Float R, T;
	R = VectorMultiply(Vec1, Vec2);		// (XX, YY, ZZ, WW)
	T = VectorSwizzle(R, 1, 0, 3, 2);	// (YY, XX, WW, ZZ)
	R = VectorAdd(R, T);				// (XX + YY, YY + XX, ZZ + WW, WW + ZZ)
	T = VectorSwizzle(R, 2, 3, 0, 1);	// (ZZ + WW, WW + ZZ, XX + YY, YY + XX)
	return VectorAdd(R, T);				// (XX + YY + ZZ + WW, YY + XX + WW + ZZ, ZZ + WW + XX + YY, WW + ZZ + YY + XX)
}

FORCEINLINE VectorRegister4Double VectorDot4(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
#if UE_PLATFORM_MATH_USE_AVX
	// AVX implementation uses fast permutes
	VectorRegister4Double R, T;
	R = VectorMultiply(Vec1, Vec2);		// (XX, YY, ZZ, WW)
	T = VectorSwizzle(R, 1, 0, 3, 2);	// (YY, XX, WW, ZZ) // fast in-lane permute
	R = VectorAdd(R, T);				// (XX + YY, YY + XX, ZZ + WW, WW + ZZ)
	T = VectorSwizzle(R, 2, 3, 0, 1);	// (ZZ + WW, WW + ZZ, XX + YY, YY + XX) // lane-swap permute
	return VectorAdd(R, T);				// (XX + YY + ZZ + WW, YY + XX + WW + ZZ, ZZ + WW + XX + YY, WW + ZZ + YY + XX)
#else
	VectorRegister2Double T, A;

	// (X, Y)
	T = _mm_mul_pd(Vec1.XY, Vec2.XY);

	// (X + Z, Y + W)
#if UE_PLATFORM_MATH_USE_FMA3
	A = _mm_fmadd_pd(Vec1.ZW, Vec2.ZW, T);
#else
	A = _mm_add_pd(_mm_mul_pd(Vec1.ZW, Vec2.ZW), T);
#endif // UE_PLATFORM_MATH_USE_FMA3

	// (Y + W, X + Z)  // Reverse of A
	T = _mm_shuffle_pd(A, A, SHUFFLEMASK2(1, 0));

	// (X + Z + Y + W, Y + W + X + Z)
	T = _mm_add_pd(A, T);
	return VectorRegister4Double(T, T);
#endif
}

/**
 * Creates a four-part mask based on component-wise == compares of the input vectors
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x == Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
 */
FORCEINLINE VectorRegister4Float VectorCompareEQ(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return _mm_cmpeq_ps(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorCompareEQ(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_cmpeq_pd(Vec1.XY, Vec2.XY);
	Result.ZW = _mm_cmpeq_pd(Vec1.ZW, Vec2.ZW);
#else
	Result = _mm256_cmp_pd(Vec1, Vec2, _CMP_EQ_OQ);
#endif
	return Result;
}

/**
 * Creates a four-part mask based on component-wise != compares of the input vectors
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x != Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
 */
FORCEINLINE VectorRegister4Float VectorCompareNE(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return _mm_cmpneq_ps(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorCompareNE(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_cmpneq_pd(Vec1.XY, Vec2.XY);
	Result.ZW = _mm_cmpneq_pd(Vec1.ZW, Vec2.ZW);
#else
	// For X != Y, if either is NaN it should return true (this matches the normal C behavior). 
	// We use the *unordered* comparison operation that is true if either value is NaN.
	Result = _mm256_cmp_pd(Vec1, Vec2, _CMP_NEQ_UQ);
#endif
	return Result;
}

/**
 * Creates a four-part mask based on component-wise > compares of the input vectors
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x > Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
 */
FORCEINLINE VectorRegister4Float VectorCompareGT(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return _mm_cmpgt_ps(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorCompareGT(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_cmpgt_pd(Vec1.XY, Vec2.XY);
	Result.ZW = _mm_cmpgt_pd(Vec1.ZW, Vec2.ZW);
#else
	Result = _mm256_cmp_pd(Vec1, Vec2, _CMP_GT_OQ);
#endif
	return Result;
}

/**
 * Creates a four-part mask based on component-wise >= compares of the input vectors
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x >= Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
 */
FORCEINLINE VectorRegister4Float VectorCompareGE(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return _mm_cmpge_ps(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorCompareGE(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_cmpge_pd(Vec1.XY, Vec2.XY);
	Result.ZW = _mm_cmpge_pd(Vec1.ZW, Vec2.ZW);
#else
	Result = _mm256_cmp_pd(Vec1, Vec2, _CMP_GE_OQ);
#endif
	return Result;
}

 /**
 * Creates a four-part mask based on component-wise < compares of the input vectors
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x < Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
 */
FORCEINLINE VectorRegister4Float VectorCompareLT(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return _mm_cmplt_ps(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorCompareLT(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_cmplt_pd(Vec1.XY, Vec2.XY);
	Result.ZW = _mm_cmplt_pd(Vec1.ZW, Vec2.ZW);
#else
	Result = _mm256_cmp_pd(Vec1, Vec2, _CMP_LT_OQ);
#endif
	return Result;
}

 /**
 * Creates a four-part mask based on component-wise <= compares of the input vectors
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( Vec1.x <= Vec2.x ? 0xFFFFFFFF : 0, same for yzw )
 */
FORCEINLINE VectorRegister4Float VectorCompareLE(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return _mm_cmple_ps(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorCompareLE(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_cmple_pd(Vec1.XY, Vec2.XY);
	Result.ZW = _mm_cmple_pd(Vec1.ZW, Vec2.ZW);
#else
	Result = _mm256_cmp_pd(Vec1, Vec2, _CMP_LE_OQ);
#endif
	return Result;
}

/**
 * Does a bitwise vector selection based on a mask (e.g., created from VectorCompareXX)
 *
 * @param Mask  Mask (when 1: use the corresponding bit from Vec1 otherwise from Vec2)
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( for each bit i: Mask[i] ? Vec1[i] : Vec2[i] )
 *
 */

FORCEINLINE VectorRegister4Float VectorSelect(VectorRegister4Float Mask, VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	// Can't (in general) use BLENDVPS despite our SSE4.1 minimum requirement since
	// this is defined to be bitwise, not element-wise with MSB as toggle
	return _mm_or_ps(_mm_and_ps(Mask, Vec1), _mm_andnot_ps(Mask, Vec2));
}

FORCEINLINE VectorRegister2Double VectorSelect(VectorRegister2Double Mask, VectorRegister2Double Vec1, VectorRegister2Double Vec2)
{
	return _mm_or_pd(_mm_and_pd(Mask, Vec1), _mm_andnot_pd(Mask, Vec2));
}

FORCEINLINE VectorRegister4Double VectorSelect(VectorRegister4Double Mask, VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = VectorSelect(Mask.XY, Vec1.XY, Vec2.XY);
	Result.ZW = VectorSelect(Mask.ZW, Vec1.ZW, Vec2.ZW);
#else
	Result = _mm256_or_pd(_mm256_and_pd(Mask, Vec1), _mm256_andnot_pd(Mask, Vec2));
#endif
	return Result;
}

/**
 * Combines two vectors using bitwise OR (treating each vector as a 128 bit field)
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( for each bit i: Vec1[i] | Vec2[i] )
 */
FORCEINLINE VectorRegister4Float VectorBitwiseOr(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return _mm_or_ps(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorBitwiseOr(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_or_pd(Vec1.XY, Vec2.XY);
	Result.ZW = _mm_or_pd(Vec1.ZW, Vec2.ZW);
#else
	Result = _mm256_or_pd(Vec1, Vec2);
#endif
	return Result;
}

/**
 * Combines two vectors using bitwise AND (treating each vector as a 128 bit field)
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( for each bit i: Vec1[i] & Vec2[i] )
 */
FORCEINLINE VectorRegister4Float VectorBitwiseAnd(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return _mm_and_ps(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorBitwiseAnd(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_and_pd(Vec1.XY, Vec2.XY);
	Result.ZW = _mm_and_pd(Vec1.ZW, Vec2.ZW);
#else
	Result = _mm256_and_pd(Vec1, Vec2);
#endif
	return Result;
}

/**
 * Combines two vectors using bitwise XOR (treating each vector as a 128 bit field)
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( for each bit i: Vec1[i] ^ Vec2[i] )
 */
FORCEINLINE VectorRegister4Float VectorBitwiseXor(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return _mm_xor_ps(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorBitwiseXor(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_xor_pd(Vec1.XY, Vec2.XY);
	Result.ZW = _mm_xor_pd(Vec1.ZW, Vec2.ZW);
#else
	Result = _mm256_xor_pd(Vec1, Vec2);
#endif
	return Result;
}

/**
 * Calculates the cross product of two vectors (XYZ components). W of the input should be 0, and will remain 0.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		cross(Vec1.xyz, Vec2.xyz). W of the input should be 0, and will remain 0.
 */
FORCEINLINE VectorRegister4Float VectorCross(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	// YZX
	VectorRegister4Float A = VectorSwizzle(Vec2, 1, 2, 0, 3);
	VectorRegister4Float B = VectorSwizzle(Vec1, 1, 2, 0, 3);
	// XY, YZ, ZX
	A = VectorMultiply(A, Vec1);
	// XY-YX, YZ-ZY, ZX-XZ
	A = VectorNegateMultiplyAdd(B, Vec2, A);
	// YZ-ZY, ZX-XZ, XY-YX
	return VectorSwizzle(A, 1, 2, 0, 3);
}

FORCEINLINE VectorRegister4Double VectorCross(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	// YZX
	VectorRegister4Double A = VectorSwizzle(Vec2, 1, 2, 0, 3);
	VectorRegister4Double B = VectorSwizzle(Vec1, 1, 2, 0, 3);
	// XY, YZ, ZX
	A = VectorMultiply(A, Vec1);
	// XY-YX, YZ-ZY, ZX-XZ
	A = VectorNegateMultiplyAdd(B, Vec2, A);
	// YZ-ZY, ZX-XZ, XY-YX
	return VectorSwizzle(A, 1, 2, 0, 3);
}

/**
 * Calculates x raised to the power of y (component-wise).
 *
 * @param Base		Base vector
 * @param Exponent	Exponent vector
 * @return			VectorRegister4Float( Base.x^Exponent.x, Base.y^Exponent.y, Base.z^Exponent.z, Base.w^Exponent.w )
 */
FORCEINLINE VectorRegister4Float VectorPow(VectorRegister4Float Base, VectorRegister4Float Exponent)
{
#if UE_PLATFORM_MATH_USE_SVML
	return _mm_pow_ps(Base, Exponent);
#else
	// using SseMath library
	return SSE::exp_ps(_mm_mul_ps(SSE::log_ps(Base), Exponent));
#endif
/*
	// old version, keeping for reference in case something breaks and we need to debug it.
	union { VectorRegister4Float v; float f[4]; } B, E;
	B.v = Base;
	E.v = Exponent;
	return _mm_setr_ps( powf(B.f[0], E.f[0]), powf(B.f[1], E.f[1]), powf(B.f[2], E.f[2]), powf(B.f[3], E.f[3]) );
*/
}

FORCEINLINE VectorRegister4Double VectorPow(VectorRegister4Double Base, VectorRegister4Double Exponent)
{
#if UE_PLATFORM_MATH_USE_SVML_AVX
	return _mm256_pow_pd(Base, Exponent);
#elif UE_PLATFORM_MATH_USE_SVML
	return VectorRegister4Double(_mm_pow_pd(Base.XY, Exponent.XY), _mm_pow_pd(Base.ZW, Exponent.ZW));
#else
	AlignedDouble4 Values(Base);
	AlignedDouble4 Exponents(Exponent);

	Values[0] = FMath::Pow(Values[0], Exponents[0]);
	Values[1] = FMath::Pow(Values[1], Exponents[1]);
	Values[2] = FMath::Pow(Values[2], Exponents[2]);
	Values[3] = FMath::Pow(Values[3], Exponents[3]);
	return Values.ToVectorRegister();
#endif
}

/**
 * Return the square root of each component
 *
 * @param Vector	Vector
 * @return			VectorRegister4Float(sqrt(Vec.X), sqrt(Vec.Y), sqrt(Vec.Z), sqrt(Vec.W))
 */
FORCEINLINE VectorRegister4Float VectorSqrt(VectorRegister4Float Vec)
{
	return _mm_sqrt_ps(Vec);
}

FORCEINLINE VectorRegister4Double VectorSqrt(VectorRegister4Double Vec)
{
#if UE_PLATFORM_MATH_USE_AVX
	return _mm256_sqrt_pd(Vec);
#else
	return VectorRegister4Double(_mm_sqrt_pd(Vec.XY), _mm_sqrt_pd(Vec.ZW));
#endif
}

/**
 * Returns an estimate of 1/sqrt(c) for each component of the vector
 *
 * @param Vector		Vector
 * @return			VectorRegister4Float(1/sqrt(t), 1/sqrt(t), 1/sqrt(t), 1/sqrt(t))
 */
FORCEINLINE VectorRegister4Float VectorReciprocalSqrtEstimate(VectorRegister4Float Vec)
{
	// Warning: Discrepancies between Intel and AMD hardware estimates make this diverge between platforms.
	return _mm_rsqrt_ps(Vec);
}

/**
 * Return the reciprocal of the square root of each component
 *
 * @param Vector		Vector 
 * @return			VectorRegister4Float(1/sqrt(Vec.X), 1/sqrt(Vec.Y), 1/sqrt(Vec.Z), 1/sqrt(Vec.W))
 */
FORCEINLINE VectorRegister4Float VectorReciprocalSqrt(VectorRegister4Float Vec)
{
#if UE_PLATFORM_MATH_USE_SVML && 0 // NOTE: DISABLED
	// TODO: this appears to deliver slightly different results on Intel vs AMD hardware,
	// similar to prior issues with our use of rsqrt refinements in UnrealPlatformMathSSE.
	return _mm_invsqrt_ps(Vec);
#else
	return VectorDivide(GlobalVectorConstants::FloatOne, VectorSqrt(Vec));
#endif

	/*
	// Legacy implementation based on refinements of estimate, left for reference.
	// Discrepancies between Intel and AMD hardware estimates make this diverge between platforms,
	// similar to prior issues with our use of rsqrt refinements in UnrealPlatformMathSSE.
	//
	// Perform two passes of Newton-Raphson iteration on the hardware estimate
	//    v^-0.5 = x
	// => x^2 = v^-1
	// => 1/(x^2) = v
	// => F(x) = x^-2 - v
	//    F'(x) = -2x^-3
	
	//    x1 = x0 - F(x0)/F'(x0)
	// => x1 = x0 + 0.5 * (x0^-2 - Vec) * x0^3
	// => x1 = x0 + 0.5 * (x0 - Vec * x0^3)
	// => x1 = x0 + x0 * (0.5 - 0.5 * Vec * x0^2)

	const VectorRegister4Float OneHalf = GlobalVectorConstants::FloatOneHalf;
	const VectorRegister4Float VecDivBy2 = VectorMultiply(Vec, OneHalf);

	// Initial estimate
	const VectorRegister4Float x0 = VectorReciprocalSqrtEstimate(Vec);

	// First iteration
	VectorRegister4Float x1 = VectorMultiply(x0, x0);
	x1 = VectorSubtract(OneHalf, VectorMultiply(VecDivBy2, x1));
	x1 = VectorMultiplyAdd(x0, x1, x0);

	// Second iteration
	VectorRegister4Float x2 = VectorMultiply(x1, x1);
	x2 = VectorSubtract(OneHalf, VectorMultiply(VecDivBy2, x2));
	x2 = VectorMultiplyAdd(x1, x2, x1);

	return x2;
	*/
}

FORCEINLINE VectorRegister4Double VectorReciprocalSqrt(VectorRegister4Double Vec)
{
#if UE_PLATFORM_MATH_USE_AVX
	return VectorDivide(GlobalVectorConstants::DoubleOne, _mm256_sqrt_pd(Vec));
#else
	return VectorDivide(GlobalVectorConstants::DoubleOne, VectorRegister4Double(_mm_sqrt_pd(Vec.XY), _mm_sqrt_pd(Vec.ZW)));
#endif
}

FORCEINLINE VectorRegister4Double VectorReciprocalSqrtEstimate(VectorRegister4Double Vec)
{
#if UE_PLATFORM_MATH_USE_SVML_AVX
	return _mm256_invsqrt_pd(Vec);
#elif UE_PLATFORM_MATH_USE_SVML
	return VectorRegister4Double(_mm_invsqrt_pd(Vec.XY), _mm_invsqrt_pd(Vec.ZW));
#else
	return VectorReciprocalSqrt(Vec);
#endif
}


/**
 * Return Reciprocal Length of the vector
 *
 * @param Vector	Vector
 * @return			VectorRegister4Float(rlen, rlen, rlen, rlen) when rlen = 1/sqrt(dot4(V))
 */
FORCEINLINE VectorRegister4Float VectorReciprocalLen(VectorRegister4Float Vector)
{
	return VectorReciprocalSqrt(VectorDot4(Vector, Vector));
}

FORCEINLINE VectorRegister4Double VectorReciprocalLen(VectorRegister4Double Vector)
{
	return VectorReciprocalSqrt(VectorDot4(Vector, Vector));
}

/**
 * Return Reciprocal Length of the vector (estimate)
 *
 * @param Vector	Vector
 * @return			VectorRegister4Float(rlen, rlen, rlen, rlen) when rlen = 1/sqrt(dot4(V))
 */
FORCEINLINE VectorRegister4Float VectorReciprocalLenEstimate(VectorRegister4Float Vector)
{
	return VectorReciprocalSqrtEstimate(VectorDot4(Vector, Vector));
}

FORCEINLINE VectorRegister4Double VectorReciprocalLenEstimate(VectorRegister4Double Vector)
{
	return VectorReciprocalSqrtEstimate(VectorDot4(Vector, Vector));
}

/**
 * Computes an estimate of the reciprocal of a vector (component-wise) and returns the result.
 *
 * @param Vec	1st vector
 * @return		VectorRegister4Float( (Estimate) 1.0f / Vec.x, (Estimate) 1.0f / Vec.y, (Estimate) 1.0f / Vec.z, (Estimate) 1.0f / Vec.w )
 */

FORCEINLINE VectorRegister4Float VectorReciprocalEstimate(VectorRegister4Float Vec)
{
	// Warning: Discrepancies between Intel and AMD hardware estimates make this diverge between platforms.
	return _mm_rcp_ps(Vec);
}

/**
 * Computes the reciprocal of a vector (component-wise) and returns the result.
 *
 * @param Vec	1st vector
 * @return		VectorRegister4Float( 1.0f / Vec.x, 1.0f / Vec.y, 1.0f / Vec.z, 1.0f / Vec.w )
 */
FORCEINLINE VectorRegister4Float VectorReciprocal(VectorRegister4Float Vec)
{
	return VectorDivide(GlobalVectorConstants::FloatOne, Vec);
	/*
	// Legacy implementation based on refinements of estimate, left for reference.
	// Discrepancies between Intel and AMD hardware estimates make this diverge between platforms.
	//
	// Perform two passes of Newton-Raphson iteration on the hardware estimate
	//   x1 = x0 - f(x0) / f'(x0)
	//
	//    1 / Vec = x
	// => x * Vec = 1 
	// => F(x) = x * Vec - 1
	//    F'(x) = Vec
	// => x1 = x0 - (x0 * Vec - 1) / Vec
	//
	// Since 1/Vec is what we're trying to solve, use an estimate for it, x0
	// => x1 = x0 - (x0 * Vec - 1) * x0 = 2 * x0 - Vec * x0^2 

	// Initial estimate
	const VectorRegister4Float x0 = VectorReciprocalEstimate(Vec);

	// First iteration
	const VectorRegister4Float x0Squared = VectorMultiply(x0, x0);
	const VectorRegister4Float x0Times2 = VectorAdd(x0, x0);
	const VectorRegister4Float x1 = VectorNegateMultiplyAdd(Vec, x0Squared, x0Times2);

	// Second iteration
	const VectorRegister4Float x1Squared = VectorMultiply(x1, x1);
	const VectorRegister4Float x1Times2 = VectorAdd(x1, x1);
	const VectorRegister4Float x2 = VectorNegateMultiplyAdd(Vec, x1Squared, x1Times2);

	return x2;
	*/
}

FORCEINLINE VectorRegister4Double VectorReciprocal(VectorRegister4Double Vec)
{
	return VectorDivide(GlobalVectorConstants::DoubleOne, Vec);
}

FORCEINLINE VectorRegister4Double VectorReciprocalEstimate(VectorRegister4Double Vec)
{
	// Not an estimate.
	return VectorReciprocal(Vec);
}

/**
 * Merges the XYZ components of one vector with the W component of another vector and returns the result.
 *
 * @param VecXYZ	Source vector for XYZ_
 * @param VecW		Source register for ___W (note: the fourth component is used, not the first)
 * @return			VectorRegister4Float(VecXYZ.x, VecXYZ.y, VecXYZ.z, VecW.w)
 */
FORCEINLINE VectorRegister4Float VectorMergeVecXYZ_VecW(VectorRegister4Float VecXYZ, VectorRegister4Float VecW)
{
	return _mm_blend_ps(VecXYZ, VecW, 0b1000);
}

FORCEINLINE VectorRegister4Double VectorMergeVecXYZ_VecW(VectorRegister4Double VecXYZ, VectorRegister4Double VecW)
{
	return VectorRegister4Double(VecXYZ.XY, _mm_move_sd(VecW.ZW, VecXYZ.ZW));
}

/**
* Loads XYZ and sets W=0
*
* @param Vector	VectorRegister4Float
* @return		VectorRegister4Float(X, Y, Z, 0.0f)
*/
FORCEINLINE VectorRegister4Float VectorSet_W0(VectorRegister4Float Vec)
{
	return _mm_insert_ps(Vec, Vec, 0x08); // Copies lane 0 to lane 0 and zero-masks lane 3
}

FORCEINLINE VectorRegister4Double VectorSet_W0(VectorRegister4Double Vec)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = Vec.XY;
	Result.ZW = _mm_move_sd(_mm_setzero_pd(), Vec.ZW);
#else
	Result = _mm256_blend_pd(Vec, VectorZeroDouble(), 0b1000);
#endif
	return Result;
}

/**
* Loads XYZ and sets W=1
*
* @param Vector	VectorRegister4Float
* @return		VectorRegister4Float(X, Y, Z, 1.0f)
*/
FORCEINLINE VectorRegister4Float VectorSet_W1(VectorRegister4Float Vec)
{
	return VectorMergeVecXYZ_VecW(Vec, VectorOneFloat());
}

FORCEINLINE VectorRegister4Double VectorSet_W1(VectorRegister4Double Vec)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = Vec.XY;
	Result.ZW = _mm_move_sd(GlobalVectorConstants::DoubleOne2d, Vec.ZW);
#else
	Result = _mm256_blend_pd(Vec, VectorOneDouble(), 0b1000);
#endif
	return Result;
}

/**
 * Multiplies two 4x4 matrices.
 *
 * @param Result	Pointer to where the result should be stored
 * @param Matrix1	Pointer to the first matrix
 * @param Matrix2	Pointer to the second matrix
 */
CORE_API void VectorMatrixMultiply(FMatrix44f* Result, const FMatrix44f* Matrix1, const FMatrix44f* Matrix2);
CORE_API void VectorMatrixMultiply(FMatrix44d* Result, const FMatrix44d* Matrix1, const FMatrix44d* Matrix2);

/**
 * Calculate the inverse of an FMatrix44.  Src == Dst is allowed
 *
 * @param DstMatrix		FMatrix44 pointer to where the result should be stored
 * @param SrcMatrix		FMatrix44 pointer to the Matrix to be inversed
 * @return bool			returns false if matrix is not invertable and stores identity 
 *
 */
FORCEINLINE bool VectorMatrixInverse(FMatrix44d* DstMatrix, const FMatrix44d* SrcMatrix)
{
	return FMath::MatrixInverse(DstMatrix,SrcMatrix);
}
FORCEINLINE bool VectorMatrixInverse(FMatrix44f* DstMatrix, const FMatrix44f* SrcMatrix)
{
	return FMath::MatrixInverse(DstMatrix,SrcMatrix);
}

/**
 * Calculate Homogeneous transform.
 *
 * @param VecP			VectorRegister4Float 
 * @param MatrixM		FMatrix pointer to the Matrix to apply transform
 * @return VectorRegister4Float = VecP*MatrixM
 */
FORCEINLINE VectorRegister4Float VectorTransformVector(VectorRegister4Float VecP, const FMatrix44f* MatrixM)
{
	const VectorRegister4Float *M = (const VectorRegister4Float *) MatrixM;
	VectorRegister4Float VTempX, VTempY, VTempZ, VTempW;

	// Splat x,y,z and w
	VTempX = VectorReplicate(VecP, 0);
	VTempY = VectorReplicate(VecP, 1);
	VTempZ = VectorReplicate(VecP, 2);
	VTempW = VectorReplicate(VecP, 3);
	// Mul by the matrix
	VTempX = VectorMultiply(VTempX, M[0]);
	VTempX = VectorMultiplyAdd(VTempY, M[1], VTempX);
	VTempX = VectorMultiplyAdd(VTempZ, M[2], VTempX);
	VTempX = VectorMultiplyAdd(VTempW, M[3], VTempX);

	return VTempX;
}

FORCEINLINE VectorRegister4Float VectorTransformVector(VectorRegister4Float VecP, const FMatrix44d* MatrixM)
{
	// Warning: FMatrix44d alignment may not match VectorRegister4Double, so you can't just cast to VectorRegister4Double*.
	typedef double Double4x4[4][4];
	const Double4x4& MRows = *((const Double4x4*)MatrixM);

	VectorRegister4Double M[4];
	M[0] = VectorLoad(MRows[0]);
	M[1] = VectorLoad(MRows[1]);
	M[2] = VectorLoad(MRows[2]);
	M[3] = VectorLoad(MRows[3]);

	VectorRegister4Double VTempX, VTempY, VTempZ, VTempW;
	VectorRegister4Double VecPDouble = VecP;

	// Splat x,y,z and w
	VTempX = VectorReplicate(VecPDouble, 0);
	VTempY = VectorReplicate(VecPDouble, 1);
	VTempZ = VectorReplicate(VecPDouble, 2);
	VTempW = VectorReplicate(VecPDouble, 3);
	// Mul by the matrix
	VTempX = VectorMultiply(VTempX, M[0]);
	VTempX = VectorMultiplyAdd(VTempY, M[1], VTempX);
	VTempX = VectorMultiplyAdd(VTempZ, M[2], VTempX);
	VTempX = VectorMultiplyAdd(VTempW, M[3], VTempX);

	// LWC_TODO: this will be a lossy conversion.
	return MakeVectorRegisterFloatFromDouble(VTempX);
}

FORCEINLINE VectorRegister4Double VectorTransformVector(VectorRegister4Double VecP, const FMatrix44d* MatrixM)
{
	// Warning: FMatrix44d alignment may not match VectorRegister4Double, so you can't just cast to VectorRegister4Double*.
	typedef double Double4x4[4][4];
	const Double4x4& MRows = *((const Double4x4*)MatrixM);

	VectorRegister4Double M[4];
	M[0] = VectorLoad(MRows[0]);
	M[1] = VectorLoad(MRows[1]);
	M[2] = VectorLoad(MRows[2]);
	M[3] = VectorLoad(MRows[3]);

	VectorRegister4Double VTempX, VTempY, VTempZ, VTempW;

	// Splat x,y,z and w
	VTempX = VectorReplicate(VecP, 0);
	VTempY = VectorReplicate(VecP, 1);
	VTempZ = VectorReplicate(VecP, 2);
	VTempW = VectorReplicate(VecP, 3);
	// Mul by the matrix
	VTempX = VectorMultiply(VTempX, M[0]);
	VTempX = VectorMultiplyAdd(VTempY, M[1], VTempX);
	VTempX = VectorMultiplyAdd(VTempZ, M[2], VTempX);
	VTempX = VectorMultiplyAdd(VTempW, M[3], VTempX);

	return VTempX;
}

FORCEINLINE VectorRegister4Double VectorTransformVector(VectorRegister4Double VecP, const FMatrix44f* MatrixM)
{
	const VectorRegister4Float* M = (const VectorRegister4Float*) MatrixM;
	VectorRegister4Double VTempX, VTempY, VTempZ, VTempW;

	// Splat x,y,z and w
	VTempX = VectorReplicate(VecP, 0);
	VTempY = VectorReplicate(VecP, 1);
	VTempZ = VectorReplicate(VecP, 2);
	VTempW = VectorReplicate(VecP, 3);
	// Mul by the matrix
	VTempX = VectorMultiply(VTempX, VectorRegister4Double(M[0]));
	VTempX = VectorMultiplyAdd(VTempY, VectorRegister4Double(M[1]), VTempX);
	VTempX = VectorMultiplyAdd(VTempZ, VectorRegister4Double(M[2]), VTempX);
	VTempX = VectorMultiplyAdd(VTempW, VectorRegister4Double(M[3]), VTempX);

	return VTempX;
}

/**
 * Returns the minimum values of two vectors (component-wise).
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( min(Vec1.x,Vec2.x), min(Vec1.y,Vec2.y), min(Vec1.z,Vec2.z), min(Vec1.w,Vec2.w) )
 */
FORCEINLINE VectorRegister4Float VectorMin(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return _mm_min_ps(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorMin(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_min_pd(Vec1.XY, Vec2.XY);
	Result.ZW = _mm_min_pd(Vec1.ZW, Vec2.ZW);
#else
	Result = _mm256_min_pd(Vec1, Vec2);
#endif
	return Result;
}

/**
 * Returns the maximum values of two vectors (component-wise).
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( max(Vec1.x,Vec2.x), max(Vec1.y,Vec2.y), max(Vec1.z,Vec2.z), max(Vec1.w,Vec2.w) )
 */
FORCEINLINE VectorRegister4Float VectorMax(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return _mm_max_ps(Vec1, Vec2);
}

FORCEINLINE VectorRegister4Double VectorMax(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_max_pd(Vec1.XY, Vec2.XY);
	Result.ZW = _mm_max_pd(Vec1.ZW, Vec2.ZW);
#else
	Result = _mm256_max_pd(Vec1, Vec2);
#endif
	return Result;
}

/**
* Creates a vector by combining two high components from each vector
*
* @param Vec1		Source vector1
* @param Vec2		Source vector2
* @return			The combined vector
*/
FORCEINLINE VectorRegister4Float VectorCombineHigh(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return VectorShuffle(Vec1, Vec2, 2, 3, 2, 3);
}

FORCEINLINE VectorRegister4Double VectorCombineHigh(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	return VectorShuffle(Vec1, Vec2, 2, 3, 2, 3);
}

/**
* Creates a vector by combining two low components from each vector
*
* @param Vec1		Source vector1
* @param Vec2		Source vector2
* @return			The combined vector
*/
FORCEINLINE VectorRegister4Float VectorCombineLow(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return VectorShuffle(Vec1, Vec2, 0, 1, 0, 1);
}

FORCEINLINE VectorRegister4Double VectorCombineLow(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	return VectorShuffle(Vec1, Vec2, 0, 1, 0, 1);
}

/**
 * Deinterleaves the components of the two given vectors such that the even components
 * are in one vector and the odds in another.
 *
 * @param Lo	[Even0, Odd0, Even1, Odd1]
 * @param Hi	[Even2, Odd2, Even3, Odd3]
 * @param OutEvens [Even0, Even1, Even2, Even3]
 * @param OutOdds [Odd0, Odd1, Odd2, Odd3]
*/
FORCEINLINE void VectorDeinterleave(VectorRegister4Float& RESTRICT OutEvens, VectorRegister4Float& RESTRICT OutOdds, VectorRegister4Float Lo, VectorRegister4Float Hi)
{
	OutEvens = VectorShuffle(Lo, Hi, 0, 2, 0, 2);
	OutOdds = VectorShuffle(Lo, Hi, 1, 3, 1, 3);
}

FORCEINLINE void VectorDeinterleave(VectorRegister4Double& RESTRICT OutEvens, VectorRegister4Double& RESTRICT OutOdds, VectorRegister4Double Lo, VectorRegister4Double Hi)
{
	OutEvens = VectorShuffle(Lo, Hi, 0, 2, 0, 2);
	OutOdds = VectorShuffle(Lo, Hi, 1, 3, 1, 3);
}


/**
 * Returns an integer bit-mask (0x00 - 0x0f) based on the sign-bit for each component in a vector.
 *
 * @param VecMask		Vector
 * @return				Bit 0 = sign(VecMask.x), Bit 1 = sign(VecMask.y), Bit 2 = sign(VecMask.z), Bit 3 = sign(VecMask.w)
 */
FORCEINLINE int VectorMaskBits(VectorRegister4Float VecMask)
{
	return _mm_movemask_ps(VecMask);
}

FORCEINLINE int VectorMaskBits(VectorRegister4Double VecMask)
{
#if !UE_PLATFORM_MATH_USE_AVX
	const int MaskXY = _mm_movemask_pd(VecMask.XY);
	const int MaskZW = _mm_movemask_pd(VecMask.ZW);
	return (MaskZW << 2) | (MaskXY);
#else
	return _mm256_movemask_pd(VecMask);
#endif
}

/**
 * Loads 4 BYTEs from unaligned memory and converts them into 4 FLOATs.
 *
 * @param Ptr			Unaligned memory pointer to the 4 BYTEs.
 * @return				VectorRegister4Float( float(Ptr[0]), float(Ptr[1]), float(Ptr[2]), float(Ptr[3]) )
 */
// Looks complex but is really quite straightforward:
// Load as 32-bit value, convert into 4x 32-bit ints, then convert to 4x floats
#define VectorLoadByte4(Ptr)			_mm_cvtepi32_ps(_mm_cvtepu8_epi32(_mm_cvtsi32_si128(*(int32*)Ptr)))

/**
* Loads 4 signed BYTEs from unaligned memory and converts them into 4 FLOATs.
*
* @param Ptr			Unaligned memory pointer to the 4 BYTEs.
* @return				VectorRegister4Float( float(Ptr[0]), float(Ptr[1]), float(Ptr[2]), float(Ptr[3]) )
*/
// Looks complex but is really quite straightforward:
// Load as 32-bit value, unpack 4x signed 8-bit ints to 4x 32-bit ints, then convert to 4x floats
FORCEINLINE VectorRegister4Float VectorLoadSignedByte4(const void* Ptr)
{
	__m128i Temp = _mm_cvtepi8_epi32(_mm_cvtsi32_si128(*(int32*)Ptr));
	return _mm_cvtepi32_ps(Temp);
}

/**
 * Loads 4 BYTEs from unaligned memory and converts them into 4 FLOATs in reversed order.
 *
 * @param Ptr			Unaligned memory pointer to the 4 BYTEs.
 * @return				VectorRegister4Float( float(Ptr[3]), float(Ptr[2]), float(Ptr[1]), float(Ptr[0]) )
 */
FORCEINLINE VectorRegister4Float VectorLoadByte4Reverse(void* Ptr)
{
	VectorRegister4Float Temp = VectorLoadByte4(Ptr);
	return VectorSwizzle( Temp, 3, 2, 1, 0 );
}

/**
 * Converts the 4 FLOATs in the vector to 4 BYTEs, clamped to [0,255], and stores to unaligned memory.
 *
 * @param Vec			Vector containing 4 FLOATs
 * @param Ptr			Unaligned memory pointer to store the 4 BYTEs.
 */
FORCEINLINE void VectorStoreByte4(VectorRegister4Float Vec, void* Ptr)
{
	// Looks complex but is really quite straightforward:
	// Convert 4x floats to 4x 32-bit ints, then pack into 4x 16-bit ints, then into 4x 8-bit unsigned ints, then store as a 32-bit value
	__m128i VecInt32 = _mm_cvttps_epi32(Vec);
	__m128i VecInt16 = _mm_packs_epi32(VecInt32, VecInt32);
	__m128i VecUInt8 = _mm_packus_epi16(VecInt16, VecInt16);
	*(int32*)Ptr = _mm_cvtsi128_si32(VecUInt8);
}

/**
* Converts the 4 FLOATs in the vector to 4 BYTEs, clamped to [-128,127], and stores to unaligned memory.
*
* @param Vec			Vector containing 4 FLOATs
* @param Ptr			Unaligned memory pointer to store the 4 BYTEs.
*/
FORCEINLINE void VectorStoreSignedByte4(VectorRegister4Float Vec, void* Ptr)
{
	// Looks complex but is really quite straightforward:
	// Convert 4x floats to 4x 32-bit ints, then pack into 4x 16-bit ints, then into 4x 8-bit unsigned ints, then store as a 32-bit value
	__m128i VecInt32 = _mm_cvttps_epi32(Vec);
	__m128i VecInt16 = _mm_packs_epi32(VecInt32, VecInt32);
	__m128i VecInt8 = _mm_packs_epi16(VecInt16, VecInt16);
	*(int32*)Ptr = _mm_cvtsi128_si32(VecInt8);
}


/**
* Loads packed RGB10A2(4 bytes) from unaligned memory and converts them into 4 FLOATs.
*
* @param Ptr			Unaligned memory pointer to the RGB10A2(4 bytes).
* @return				VectorRegister4Float with 4 FLOATs loaded from Ptr.
*/
FORCEINLINE VectorRegister4Float VectorLoadURGB10A2N(void* Ptr)
{
	VectorRegister4Float Tmp;

	Tmp = _mm_and_ps(_mm_load_ps1((const float *)Ptr), MakeVectorRegisterFloat(0x3FFu, 0x3FFu << 10, 0x3FFu << 20, 0x3u << 30));
	Tmp = _mm_xor_ps(Tmp, MakeVectorRegister(0, 0, 0, 0x80000000));
	Tmp = _mm_cvtepi32_ps(*(const VectorRegister4Int*)&Tmp);
	Tmp = _mm_add_ps(Tmp, MakeVectorRegister(0, 0, 0, 32768.0f*65536.0f));
	Tmp = _mm_mul_ps(Tmp, MakeVectorRegister(1.0f / 1023.0f, 1.0f / (1023.0f*1024.0f), 1.0f / (1023.0f*1024.0f*1024.0f), 1.0f / (3.0f*1024.0f*1024.0f*1024.0f)));

	return Tmp;
}

/**
* Converts the 4 FLOATs in the vector RGB10A2, clamped to [0, 1023] and [0, 3], and stores to unaligned memory.
*
* @param Vec			Vector containing 4 FLOATs
* @param Ptr			Unaligned memory pointer to store the packed RGBA16(8 bytes).
*/
FORCEINLINE void VectorStoreURGB10A2N(VectorRegister4Float Vec, void* Ptr)
{
	VectorRegister4Float Tmp;
	Tmp = _mm_max_ps(Vec, MakeVectorRegisterFloat(0.0f, 0.0f, 0.0f, 0.0f));
	Tmp = _mm_min_ps(Tmp, MakeVectorRegisterFloat(1.0f, 1.0f, 1.0f, 1.0f));
	Tmp = _mm_mul_ps(Tmp, MakeVectorRegisterFloat(1023.0f, 1023.0f*1024.0f*0.5f, 1023.0f*1024.0f*1024.0f, 3.0f*1024.0f*1024.0f*1024.0f*0.5f));

	VectorRegister4Int TmpI;
	TmpI = _mm_cvttps_epi32(Tmp);
	TmpI = _mm_and_si128(TmpI, MakeVectorRegisterInt(0x3FFu, 0x3FFu << (10 - 1), 0x3FFu << 20, 0x3u << (30 - 1)));

	VectorRegister4Int TmpI2;
	TmpI2 = _mm_shuffle_epi32(TmpI, _MM_SHUFFLE(3, 2, 3, 2));
	TmpI = _mm_or_si128(TmpI, TmpI2);

	TmpI2 = _mm_shuffle_epi32(TmpI, _MM_SHUFFLE(1, 1, 1, 1));
	TmpI2 = _mm_add_epi32(TmpI2, TmpI2);
	TmpI = _mm_or_si128(TmpI, TmpI2);

	_mm_store_ss((float *)Ptr, *(const VectorRegister4Float*)&TmpI);
}

/**
 * Loads packed RGBA16(8 bytes) from unaligned memory and converts them into 4 FLOATs.
 *
 * @param Ptr			Unaligned memory pointer to the RGBA16(8 bytes).
 * @return				VectorRegister4Float with 4 FLOATs loaded from Ptr.
 */
#define VectorLoadURGBA16N(Ptr) _mm_cvtepi32_ps(_mm_cvtepu16_epi32(_mm_loadl_epi64((const __m128i*)Ptr)))

/**
 * Loads packed signed RGBA16(8 bytes) from unaligned memory and converts them into 4 FLOATs.
 *
 * @param Ptr			Unaligned memory pointer to the RGBA16(8 bytes).
 * @return				VectorRegister4Float with 4 FLOATs loaded from Ptr.
 */
FORCEINLINE VectorRegister4Float VectorLoadSRGBA16N(const void* Ptr)
{
	__m128i Temp = _mm_cvtepi16_epi32(_mm_loadl_epi64((const __m128i*)Ptr));
	return _mm_cvtepi32_ps(Temp);
}

/**
* Converts the 4 FLOATs in the vector to RGBA16, clamped to [0, 65535], and stores to unaligned memory.
*
* @param Vec			Vector containing 4 FLOATs
* @param Ptr			Unaligned memory pointer to store the packed RGBA16(8 bytes).
*/
FORCEINLINE void VectorStoreURGBA16N(VectorRegister4Float Vec, void* Ptr)
{
	VectorRegister4Float Tmp;
	Tmp = _mm_max_ps(Vec, MakeVectorRegisterFloat(0.0f, 0.0f, 0.0f, 0.0f));
	Tmp = _mm_min_ps(Tmp, MakeVectorRegisterFloat(1.0f, 1.0f, 1.0f, 1.0f));
	Tmp = _mm_mul_ps(Tmp, MakeVectorRegisterFloat(65535.0f, 65535.0f, 65535.0f, 65535.0f));

	VectorRegister4Int TmpI = _mm_cvtps_epi32(Tmp);
	_mm_storel_epi64((__m128i*)Ptr, _mm_packus_epi32(TmpI, TmpI));
}

/**
 * Returns non-zero if any element in Vec1 is greater than the corresponding element in Vec2, otherwise 0.
 *
 * @param Vec1			1st source vector
 * @param Vec2			2nd source vector
 * @return				Non-zero integer if (Vec1.x > Vec2.x) || (Vec1.y > Vec2.y) || (Vec1.z > Vec2.z) || (Vec1.w > Vec2.w)
 */
FORCEINLINE int VectorAnyGreaterThan(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return VectorMaskBits(VectorCompareGT(Vec1, Vec2));
}

FORCEINLINE int VectorAnyGreaterThan(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	return VectorMaskBits(VectorCompareGT(Vec1, Vec2));
}

/**
 * Resets the floating point registers so that they can be used again.
 * Some intrinsics use these for MMX purposes (e.g. VectorLoadByte4 and VectorStoreByte4).
 */
// This is no longer necessary now that we don't use MMX instructions
#define VectorResetFloatRegisters()
// TODO: LWC: remove?

/**
 * Returns the control register.
 *
 * @return			The uint32 control register
 */
#define VectorGetControlRegister()		_mm_getcsr()

#if PLATFORM_SUPPORTS_VECTOR_CONTROL_REGISTERS
/**
 * Sets the control register.
 *
 * @param ControlStatus		The uint32 control status value to set
 */
#define	VectorSetControlRegister(ControlStatus) _mm_setcsr( ControlStatus )

/**
 * Control status bit to round all floating point math results towards zero.
 */
#define VECTOR_ROUND_TOWARD_ZERO		_MM_ROUND_TOWARD_ZERO

 /**
  * Denormal operands and results will be flushed to zero
  */
#define VECTOR_DENORMALS_FLUSH_TO_ZERO	_MM_FLUSH_ZERO_ON

#else
#define VectorSetControlRegister(...)
#define VECTOR_ROUND_TOWARD_ZERO
#define VECTOR_DENORMALS_FLUSH_TO_ZERO
#endif

/**
* Multiplies two quaternions; the order matters.
*
* Order matters when composing quaternions: C = VectorQuaternionMultiply2(A, B) will yield a quaternion C = A * B
* that logically first applies B then A to any subsequent transformation (right first, then left).
*
* @param Quat1	Pointer to the first quaternion
* @param Quat2	Pointer to the second quaternion
* @return Quat1 * Quat2
*/
FORCEINLINE VectorRegister4Float VectorQuaternionMultiply2(VectorRegister4Float Quat1, VectorRegister4Float Quat2)
{
	VectorRegister4Float Result = VectorMultiply(VectorReplicate(Quat1, 3), Quat2);
	Result = VectorMultiplyAdd( VectorMultiply(VectorReplicate(Quat1, 0), VectorSwizzle(Quat2, 3, 2, 1, 0)), GlobalVectorConstants::QMULTI_SIGN_MASK0, Result);
	Result = VectorMultiplyAdd( VectorMultiply(VectorReplicate(Quat1, 1), VectorSwizzle(Quat2, 2, 3, 0, 1)), GlobalVectorConstants::QMULTI_SIGN_MASK1, Result);
	Result = VectorMultiplyAdd( VectorMultiply(VectorReplicate(Quat1, 2), VectorSwizzle(Quat2, 1, 0, 3, 2)), GlobalVectorConstants::QMULTI_SIGN_MASK2, Result);

	return Result;
}

FORCEINLINE VectorRegister4Double VectorQuaternionMultiply2(VectorRegister4Double Quat1, VectorRegister4Double Quat2)
{
	VectorRegister4Double Result = VectorMultiply(VectorReplicate(Quat1, 3), Quat2);
	Result = VectorMultiplyAdd(VectorMultiply(VectorReplicate(Quat1, 0), VectorSwizzle(Quat2, 3, 2, 1, 0)), GlobalVectorConstants::DOUBLE_QMULTI_SIGN_MASK0, Result);
	Result = VectorMultiplyAdd(VectorMultiply(VectorReplicate(Quat1, 1), VectorSwizzle(Quat2, 2, 3, 0, 1)), GlobalVectorConstants::DOUBLE_QMULTI_SIGN_MASK1, Result);
	Result = VectorMultiplyAdd(VectorMultiply(VectorReplicate(Quat1, 2), VectorSwizzle(Quat2, 1, 0, 3, 2)), GlobalVectorConstants::DOUBLE_QMULTI_SIGN_MASK2, Result);

	return Result;
}

/**
* Multiplies two quaternions; the order matters.
*
* When composing quaternions: VectorQuaternionMultiply(C, A, B) will yield a quaternion C = A * B
* that logically first applies B then A to any subsequent transformation (right first, then left).
*
* @param Result	Pointer to where the result Quat1 * Quat2 should be stored
* @param Quat1	Pointer to the first quaternion (must not be the destination)
* @param Quat2	Pointer to the second quaternion (must not be the destination)
*/
FORCEINLINE void VectorQuaternionMultiply(VectorRegister4Float* RESTRICT Result, const VectorRegister4Float* RESTRICT Quat1, const VectorRegister4Float* RESTRICT Quat2)
{
	*Result = VectorQuaternionMultiply2(*Quat1, *Quat2);
}

FORCEINLINE void VectorQuaternionMultiply(VectorRegister4Double* RESTRICT Result, const VectorRegister4Double* RESTRICT Quat1, const VectorRegister4Double* RESTRICT Quat2)
{
	*Result = VectorQuaternionMultiply2(*Quat1, *Quat2);
}

// Returns true if the vector contains a component that is either NAN or +/-infinite.
FORCEINLINE bool VectorContainsNaNOrInfinite(VectorRegister4Float Vec)
{
	// https://en.wikipedia.org/wiki/IEEE_754-1985
	// Infinity is represented with all exponent bits set, with the correct sign bit.
	// NaN is represented with all exponent bits set, plus at least one fraction/significand bit set.
	// This means finite values will not have all exponent bits set, so check against those bits.

	union { float F; uint32 U; } InfUnion;
	InfUnion.U = 0x7F800000;
	const float Inf = InfUnion.F;
	const VectorRegister4Float FloatInfinity = MakeVectorRegisterFloat(Inf, Inf, Inf, Inf);

	// Mask off Exponent
	VectorRegister4Float ExpTest = VectorBitwiseAnd(Vec, FloatInfinity);
	// Compare to full exponent. If any are full exponent (not finite), the signs copied to the mask are non-zero, otherwise it's zero and finite.
	bool IsFinite = VectorMaskBits(VectorCompareEQ(ExpTest, FloatInfinity)) == 0;
	return !IsFinite;
}

FORCEINLINE bool VectorContainsNaNOrInfinite(VectorRegister4Double Vec)
{
	// https://en.wikipedia.org/wiki/IEEE_754-1985
	// Infinity is represented with all exponent bits set, with the correct sign bit.
	// NaN is represented with all exponent bits set, plus at least one fraction/significand bit set.
	// This means finite values will not have all exponent bits set, so check against those bits.

	union { double D; uint64 U; } InfUnion;
	InfUnion.U = 0x7FF0000000000000;
	const double Inf = InfUnion.D;
	const VectorRegister4Double DoubleInfinity = MakeVectorRegisterDouble(Inf, Inf, Inf, Inf);

	// Mask off Exponent
	VectorRegister4Double ExpTest = VectorBitwiseAnd(Vec, DoubleInfinity);
	// Compare to full exponent. If any are full exponent (not finite), the signs copied to the mask are non-zero, otherwise it's zero and finite.
	bool IsFinite = VectorMaskBits(VectorCompareEQ(ExpTest, DoubleInfinity)) == 0;
	return !IsFinite;
}

FORCEINLINE VectorRegister4Float VectorTruncate(VectorRegister4Float Vec)
{
	return _mm_round_ps(Vec, _MM_FROUND_TRUNC);
}

FORCEINLINE VectorRegister2Double TruncateVectorRegister2d(VectorRegister2Double V)
{
	return _mm_round_pd(V, _MM_FROUND_TRUNC);
}

FORCEINLINE VectorRegister4Double VectorTruncate(VectorRegister4Double V)
{
	VectorRegister4Double Result;
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = TruncateVectorRegister2d(V.XY);
	Result.ZW = TruncateVectorRegister2d(V.ZW);
#else
	Result = _mm256_round_pd(V, _MM_FROUND_TO_ZERO | _MM_FROUND_NO_EXC);
#endif
	return Result;
}

FORCEINLINE VectorRegister4Float VectorRound(VectorRegister4Float Vec)
{
	return _mm_round_ps(Vec, _MM_FROUND_TO_NEAREST_INT |_MM_FROUND_NO_EXC);
}

FORCEINLINE VectorRegister4Int VectorRoundToIntHalfToEven(VectorRegister4Float Vec)
{
	return _mm_cvtps_epi32(Vec);
}

FORCEINLINE VectorRegister4Float VectorCeil(VectorRegister4Float V)
{
	return _mm_ceil_ps(V);
}

FORCEINLINE VectorRegister4Double VectorCeil(VectorRegister4Double V)
{
#if UE_PLATFORM_MATH_USE_AVX
	VectorRegister4Double Result;
	Result = _mm256_round_pd(V, _MM_FROUND_TO_POS_INF | _MM_FROUND_NO_EXC);
	return Result;
#else
	VectorRegister4Double Result;
	Result.XY = _mm_ceil_pd(V.XY);
	Result.ZW = _mm_ceil_pd(V.ZW);
	return Result;
#endif
}

FORCEINLINE VectorRegister4Float VectorFloor(VectorRegister4Float V)
{
	return _mm_floor_ps(V);
}

FORCEINLINE VectorRegister4Double VectorFloor(VectorRegister4Double V)
{
#if UE_PLATFORM_MATH_USE_AVX
	VectorRegister4Double Result;
	Result = _mm256_round_pd(V, _MM_FROUND_TO_NEG_INF | _MM_FROUND_NO_EXC);
	return Result;
#else
	VectorRegister4Double Result;
	Result.XY = _mm_floor_pd(V.XY);
	Result.ZW = _mm_floor_pd(V.ZW);
	return Result;
#endif
}

FORCEINLINE VectorRegister4Float VectorMod(VectorRegister4Float X, VectorRegister4Float Y)
{
	// Check against invalid divisor
	VectorRegister4Float InvalidDivisorMask = VectorCompareLE(VectorAbs(Y), GlobalVectorConstants::SmallNumber);

#if UE_PLATFORM_MATH_USE_SVML
	VectorRegister4Float Result = _mm_fmod_ps(X, Y);
#else
	AlignedFloat4 XFloats(X), YFloats(Y);
	XFloats[0] = fmodf(XFloats[0], YFloats[0]);
	XFloats[1] = fmodf(XFloats[1], YFloats[1]);
	XFloats[2] = fmodf(XFloats[2], YFloats[2]);
	XFloats[3] = fmodf(XFloats[3], YFloats[3]);
	VectorRegister4Float Result = XFloats.ToVectorRegister();
#endif

	// Return 0 where divisor Y was too small	
	Result = VectorSelect(InvalidDivisorMask, GlobalVectorConstants::FloatZero, Result);
	return Result;
}

FORCEINLINE VectorRegister4Double VectorMod(VectorRegister4Double X, VectorRegister4Double Y)
{
	// Check against invalid divisor
	VectorRegister4Double InvalidDivisorMask = VectorCompareLE(VectorAbs(Y), GlobalVectorConstants::DoubleSmallNumber);

#if UE_PLATFORM_MATH_USE_SVML_AVX
	VectorRegister4Double DoubleResult = _mm256_fmod_pd(X, Y);
#elif UE_PLATFORM_MATH_USE_SVML
	VectorRegister4Double DoubleResult = VectorRegister4Double(_mm_fmod_pd(X.XY, Y.XY), _mm_fmod_pd(X.ZW, Y.ZW));
#else
	AlignedDouble4 XDoubles(X), YDoubles(Y);
	XDoubles[0] = fmod(XDoubles[0], YDoubles[0]);
	XDoubles[1] = fmod(XDoubles[1], YDoubles[1]);
	XDoubles[2] = fmod(XDoubles[2], YDoubles[2]);
	XDoubles[3] = fmod(XDoubles[3], YDoubles[3]);
	VectorRegister4Double DoubleResult = XDoubles.ToVectorRegister();
#endif

	// Return 0 where divisor Y was too small
	DoubleResult = VectorSelect(InvalidDivisorMask, GlobalVectorConstants::DoubleZero, DoubleResult);
	return DoubleResult;
}

FORCEINLINE VectorRegister4Float VectorSign(VectorRegister4Float X)
{
	VectorRegister4Float Mask = VectorCompareGE(X, GlobalVectorConstants::FloatZero);
	return VectorSelect(Mask, GlobalVectorConstants::FloatOne, GlobalVectorConstants::FloatMinusOne);
}

FORCEINLINE VectorRegister4Double VectorSign(VectorRegister4Double X)
{
	VectorRegister4Double Mask = VectorCompareGE(X, (GlobalVectorConstants::DoubleZero));
	return VectorSelect(Mask, GlobalVectorConstants::DoubleOne, GlobalVectorConstants::DoubleMinusOne);
}

FORCEINLINE VectorRegister4Float VectorStep(VectorRegister4Float X)
{
	VectorRegister4Float Mask = VectorCompareGE(X, GlobalVectorConstants::FloatZero);
	return VectorSelect(Mask, GlobalVectorConstants::FloatOne, GlobalVectorConstants::FloatZero);
}

FORCEINLINE VectorRegister4Double VectorStep(VectorRegister4Double X)
{
	VectorRegister4Double Mask = VectorCompareGE(X, GlobalVectorConstants::DoubleZero);
	return VectorSelect(Mask, GlobalVectorConstants::DoubleOne, GlobalVectorConstants::DoubleZero);
}

FORCEINLINE VectorRegister4Float VectorExp(VectorRegister4Float X)
{
#if UE_PLATFORM_MATH_USE_SVML
	return _mm_exp_ps(X);
#else
	return SSE::exp_ps(X);
#endif
}

FORCEINLINE VectorRegister4Double VectorExp(VectorRegister4Double X)
{
#if UE_PLATFORM_MATH_USE_SVML_AVX
	return _mm256_exp_pd(X);
#elif UE_PLATFORM_MATH_USE_SVML
	return VectorRegister4Double(_mm_exp_pd(X.XY), _mm_exp_pd(X.ZW));
#else
	AlignedDouble4 Doubles(X);
	Doubles[0] = FMath::Exp(Doubles[0]);
	Doubles[1] = FMath::Exp(Doubles[1]);
	Doubles[2] = FMath::Exp(Doubles[2]);
	Doubles[3] = FMath::Exp(Doubles[3]);
	return Doubles.ToVectorRegister();
#endif
}

FORCEINLINE VectorRegister4Float VectorExp2(VectorRegister4Float X)
{
#if UE_PLATFORM_MATH_USE_SVML
	return _mm_exp2_ps(X);
#else
	AlignedFloat4 Floats(X);
	Floats[0] = FMath::Exp2(Floats[0]);
	Floats[1] = FMath::Exp2(Floats[1]);
	Floats[2] = FMath::Exp2(Floats[2]);
	Floats[3] = FMath::Exp2(Floats[3]);
	return Floats.ToVectorRegister();
#endif
}

FORCEINLINE VectorRegister4Double VectorExp2(VectorRegister4Double X)
{
#if UE_PLATFORM_MATH_USE_SVML_AVX
	return _mm256_exp2_pd(X);
#elif UE_PLATFORM_MATH_USE_SVML
	return VectorRegister4Double(_mm_exp2_pd(X.XY), _mm_exp2_pd(X.ZW));
#else
	AlignedDouble4 Doubles(X);
	Doubles[0] = FMath::Exp2(Doubles[0]);
	Doubles[1] = FMath::Exp2(Doubles[1]);
	Doubles[2] = FMath::Exp2(Doubles[2]);
	Doubles[3] = FMath::Exp2(Doubles[3]);
	return Doubles.ToVectorRegister();
#endif
}

FORCEINLINE VectorRegister4Float VectorLog(VectorRegister4Float X)
{
#if UE_PLATFORM_MATH_USE_SVML
	return _mm_log_ps(X);
#else
	return SSE::log_ps(X);
#endif
}

FORCEINLINE VectorRegister4Double VectorLog(VectorRegister4Double X)
{
#if UE_PLATFORM_MATH_USE_SVML_AVX
	return _mm256_log_pd(X);
#elif UE_PLATFORM_MATH_USE_SVML
	return VectorRegister4Double(_mm_log_pd(X.XY), _mm_log_pd(X.ZW));
#else
	AlignedDouble4 Doubles(X);
	Doubles[0] = FMath::Loge(Doubles[0]);
	Doubles[1] = FMath::Loge(Doubles[1]);
	Doubles[2] = FMath::Loge(Doubles[2]);
	Doubles[3] = FMath::Loge(Doubles[3]);
	return Doubles.ToVectorRegister();
#endif
}

FORCEINLINE VectorRegister4Float VectorLog2(VectorRegister4Float X)
{
#if UE_PLATFORM_MATH_USE_SVML
	return _mm_log2_ps(X);
#else
	AlignedFloat4 Floats(X);
	Floats[0] = FMath::Log2(Floats[0]);
	Floats[1] = FMath::Log2(Floats[1]);
	Floats[2] = FMath::Log2(Floats[2]);
	Floats[3] = FMath::Log2(Floats[3]);
	return Floats.ToVectorRegister();
#endif
}

FORCEINLINE VectorRegister4Double VectorLog2(VectorRegister4Double X)
{
#if UE_PLATFORM_MATH_USE_SVML_AVX
	return _mm256_log2_pd(X);
#elif UE_PLATFORM_MATH_USE_SVML
	return VectorRegister4Double(_mm_log2_pd(X.XY), _mm_log2_pd(X.ZW));
#else
	AlignedDouble4 Doubles(X);
	Doubles[0] = FMath::Log2(Doubles[0]);
	Doubles[1] = FMath::Log2(Doubles[1]);
	Doubles[2] = FMath::Log2(Doubles[2]);
	Doubles[3] = FMath::Log2(Doubles[3]);
	return Doubles.ToVectorRegister();
#endif
}


/**
 * Using "static const float ..." or "static const VectorRegister4Float ..." in functions creates the branch and code to construct those constants.
 * Doing this in FORCEINLINE not only means you introduce a branch per static, but you bloat the inlined code immensely.
 * Defining these constants at the global scope causes them to be created at startup, and avoids the cost at the function level.
 * Doing it at the function level is okay for anything that is a simple "const float", but usage of "sqrt()" here forces actual function calls.
 */
namespace VectorSinConstantsSSE
{
	static const float p = 0.225f;
	static const float a = 7.58946609f; // 16 * sqrtf(p)
	static const float b = 1.63384342f; // (1 - p) / sqrtf(p)
	static const VectorRegister4Float A = MakeVectorRegisterFloatConstant(a, a, a, a);
	static const VectorRegister4Float B = MakeVectorRegisterFloatConstant(b, b, b, b);
}

FORCEINLINE VectorRegister4Float VectorSin(VectorRegister4Float V)
{
#if UE_PLATFORM_MATH_USE_SVML
	return _mm_sin_ps(V);
#else
	return SSE::sin_ps(V);
#endif
}

FORCEINLINE VectorRegister4Double VectorSin(VectorRegister4Double V)
{
#if UE_PLATFORM_MATH_USE_SVML_AVX
	return _mm256_sin_pd(V);
#elif UE_PLATFORM_MATH_USE_SVML
	return VectorRegister4Double(_mm_sin_pd(V.XY), _mm_sin_pd(V.ZW));
#else
	AlignedDouble4 Doubles(V);
	Doubles[0] = FMath::Sin(Doubles[0]);
	Doubles[1] = FMath::Sin(Doubles[1]);
	Doubles[2] = FMath::Sin(Doubles[2]);
	Doubles[3] = FMath::Sin(Doubles[3]);
	return Doubles.ToVectorRegister();
#endif
}

FORCEINLINE VectorRegister4Float VectorCos(VectorRegister4Float V)
{
#if UE_PLATFORM_MATH_USE_SVML
	return _mm_cos_ps(V);
#else
	return SSE::cos_ps(V);
#endif
}

FORCEINLINE VectorRegister4Double VectorCos(VectorRegister4Double V)
{
#if UE_PLATFORM_MATH_USE_SVML_AVX
	return _mm256_cos_pd(V);
#elif UE_PLATFORM_MATH_USE_SVML
	return VectorRegister4Double(_mm_cos_pd(V.XY), _mm_cos_pd(V.ZW));
#else
	AlignedDouble4 Doubles(V);
	Doubles[0] = FMath::Cos(Doubles[0]);
	Doubles[1] = FMath::Cos(Doubles[1]);
	Doubles[2] = FMath::Cos(Doubles[2]);
	Doubles[3] = FMath::Cos(Doubles[3]);
	return Doubles.ToVectorRegister();
#endif
}

/**
* Computes the sine and cosine of each component of a Vector.
*
* @param VSinAngles	VectorRegister4Float Pointer to where the Sin result should be stored
* @param VCosAngles	VectorRegister4Float Pointer to where the Cos result should be stored
* @param VAngles VectorRegister4Float Pointer to the input angles
*/
FORCEINLINE void VectorSinCos(VectorRegister4Float* RESTRICT VSinAngles, VectorRegister4Float* RESTRICT VCosAngles, const VectorRegister4Float* RESTRICT VAngles)
{
#if UE_PLATFORM_MATH_USE_SVML
	*VSinAngles = _mm_sincos_ps(VCosAngles, *VAngles);
#else
	// Map to [-pi, pi]
	// X = A - 2pi * round(A/2pi)
	// Note the round(), not truncate(). In this case round() can round halfway cases using round-to-nearest-even OR round-to-nearest.

	// Quotient = round(A/2pi)
	VectorRegister4Float Quotient = VectorMultiply(*VAngles, GlobalVectorConstants::OneOverTwoPi);
	Quotient = _mm_cvtepi32_ps(_mm_cvtps_epi32(Quotient)); // round to nearest even is the default rounding mode but that's fine here.
	// X = A - 2pi * Quotient
	VectorRegister4Float X = VectorNegateMultiplyAdd(GlobalVectorConstants::TwoPi, Quotient, *VAngles);

	// Map in [-pi/2,pi/2]
	VectorRegister4Float sign = VectorBitwiseAnd(X, GlobalVectorConstants::SignBit());
	VectorRegister4Float c = VectorBitwiseOr(GlobalVectorConstants::Pi, sign);  // pi when x >= 0, -pi when x < 0
	VectorRegister4Float absx = VectorAbs(X);
	VectorRegister4Float rflx = VectorSubtract(c, X);
	VectorRegister4Float comp = VectorCompareGT(absx, GlobalVectorConstants::PiByTwo);
	X = VectorSelect(comp, rflx, X);
	sign = VectorSelect(comp, GlobalVectorConstants::FloatMinusOne, GlobalVectorConstants::FloatOne);

	const VectorRegister4Float XSquared = VectorMultiply(X, X);

	// 11-degree minimax approximation
	//*ScalarSin = (((((-2.3889859e-08f * y2 + 2.7525562e-06f) * y2 - 0.00019840874f) * y2 + 0.0083333310f) * y2 - 0.16666667f) * y2 + 1.0f) * y;
	const VectorRegister4Float SinCoeff0 = MakeVectorRegisterFloat(1.0f, -0.16666667f, 0.0083333310f, -0.00019840874f);
	const VectorRegister4Float SinCoeff1 = MakeVectorRegisterFloat(2.7525562e-06f, -2.3889859e-08f, /*unused*/ 0.f, /*unused*/ 0.f);

	VectorRegister4Float S;
	S = VectorReplicate(SinCoeff1, 1);
	S = VectorMultiplyAdd(XSquared, S, VectorReplicate(SinCoeff1, 0));
	S = VectorMultiplyAdd(XSquared, S, VectorReplicate(SinCoeff0, 3));
	S = VectorMultiplyAdd(XSquared, S, VectorReplicate(SinCoeff0, 2));
	S = VectorMultiplyAdd(XSquared, S, VectorReplicate(SinCoeff0, 1));
	S = VectorMultiplyAdd(XSquared, S, VectorReplicate(SinCoeff0, 0));
	*VSinAngles = VectorMultiply(S, X);

	// 10-degree minimax approximation
	//*ScalarCos = sign * (((((-2.6051615e-07f * y2 + 2.4760495e-05f) * y2 - 0.0013888378f) * y2 + 0.041666638f) * y2 - 0.5f) * y2 + 1.0f);
	const VectorRegister4Float CosCoeff0 = MakeVectorRegisterFloat(1.0f, -0.5f, 0.041666638f, -0.0013888378f);
	const VectorRegister4Float CosCoeff1 = MakeVectorRegisterFloat(2.4760495e-05f, -2.6051615e-07f, /*unused*/ 0.f, /*unused*/ 0.f);

	VectorRegister4Float C;
	C = VectorReplicate(CosCoeff1, 1);
	C = VectorMultiplyAdd(XSquared, C, VectorReplicate(CosCoeff1, 0));
	C = VectorMultiplyAdd(XSquared, C, VectorReplicate(CosCoeff0, 3));
	C = VectorMultiplyAdd(XSquared, C, VectorReplicate(CosCoeff0, 2));
	C = VectorMultiplyAdd(XSquared, C, VectorReplicate(CosCoeff0, 1));
	C = VectorMultiplyAdd(XSquared, C, VectorReplicate(CosCoeff0, 0));
	*VCosAngles = VectorMultiply(C, sign);
#endif
}

FORCEINLINE void VectorSinCos(VectorRegister4Double* RESTRICT VSinAngles, VectorRegister4Double* RESTRICT VCosAngles, const VectorRegister4Double* RESTRICT VAngles)
{
#if UE_PLATFORM_MATH_USE_SVML_AVX
	VSinAngles->XYZW = _mm256_sincos_pd(&(VCosAngles->XYZW), VAngles->XYZW);
#elif UE_PLATFORM_MATH_USE_SVML
	VSinAngles->XY = _mm_sincos_pd(&(VCosAngles->XY), VAngles->XY);
	VSinAngles->ZW = _mm_sincos_pd(&(VCosAngles->ZW), VAngles->ZW);
#else
	*VSinAngles = VectorSin(*VAngles);
	*VCosAngles = VectorCos(*VAngles);
#endif
}


FORCEINLINE VectorRegister4Float VectorTan(VectorRegister4Float X)
{
#if UE_PLATFORM_MATH_USE_SVML
	return _mm_tan_ps(X);
#else
	//return SSE::tan_ps(X);
	AlignedFloat4 Floats(X);
	Floats[0] = FMath::Tan(Floats[0]);
	Floats[1] = FMath::Tan(Floats[1]);
	Floats[2] = FMath::Tan(Floats[2]);
	Floats[3] = FMath::Tan(Floats[3]);
	return Floats.ToVectorRegister();
#endif
}

FORCEINLINE VectorRegister4Double VectorTan(VectorRegister4Double X)
{
#if UE_PLATFORM_MATH_USE_SVML_AVX
	return _mm256_tan_pd(X);
#elif UE_PLATFORM_MATH_USE_SVML
	return VectorRegister4Double(_mm_tan_pd(X.XY), _mm_tan_pd(X.ZW));
#else
	AlignedDouble4 Doubles(X);
	Doubles[0] = FMath::Tan(Doubles[0]);
	Doubles[1] = FMath::Tan(Doubles[1]);
	Doubles[2] = FMath::Tan(Doubles[2]);
	Doubles[3] = FMath::Tan(Doubles[3]);
	return Doubles.ToVectorRegister();
#endif
}

FORCEINLINE VectorRegister4Float VectorASin(VectorRegister4Float X)
{
#if UE_PLATFORM_MATH_USE_SVML
	return _mm_asin_ps(X);
#else
	AlignedFloat4 Floats(X);
	Floats[0] = FMath::Asin(Floats[0]);
	Floats[1] = FMath::Asin(Floats[1]);
	Floats[2] = FMath::Asin(Floats[2]);
	Floats[3] = FMath::Asin(Floats[3]);
	return Floats.ToVectorRegister();
#endif
}

FORCEINLINE VectorRegister4Double VectorASin(VectorRegister4Double X)
{
#if UE_PLATFORM_MATH_USE_SVML_AVX
	return _mm256_asin_pd(X);
#elif UE_PLATFORM_MATH_USE_SVML
	return VectorRegister4Double(_mm_asin_pd(X.XY), _mm_asin_pd(X.ZW));
#else
	AlignedDouble4 Doubles(X);
	Doubles[0] = FMath::Asin(Doubles[0]);
	Doubles[1] = FMath::Asin(Doubles[1]);
	Doubles[2] = FMath::Asin(Doubles[2]);
	Doubles[3] = FMath::Asin(Doubles[3]);
	return Doubles.ToVectorRegister();
#endif
}

FORCEINLINE VectorRegister4Float VectorACos(VectorRegister4Float X)
{
#if UE_PLATFORM_MATH_USE_SVML
	return _mm_acos_ps(X);
#else
	AlignedFloat4 Floats(X);
	Floats[0] = FMath::Acos(Floats[0]);
	Floats[1] = FMath::Acos(Floats[1]);
	Floats[2] = FMath::Acos(Floats[2]);
	Floats[3] = FMath::Acos(Floats[3]);
	return Floats.ToVectorRegister();
#endif
}

FORCEINLINE VectorRegister4Double VectorACos(VectorRegister4Double X)
{
#if UE_PLATFORM_MATH_USE_SVML_AVX
	return _mm256_acos_pd(X);
#elif UE_PLATFORM_MATH_USE_SVML
	return VectorRegister4Double(_mm_acos_pd(X.XY), _mm_acos_pd(X.ZW));
#else
	AlignedDouble4 Doubles(X);
	Doubles[0] = FMath::Acos(Doubles[0]);
	Doubles[1] = FMath::Acos(Doubles[1]);
	Doubles[2] = FMath::Acos(Doubles[2]);
	Doubles[3] = FMath::Acos(Doubles[3]);
	return Doubles.ToVectorRegister();
#endif
}

FORCEINLINE VectorRegister4Float VectorATan(VectorRegister4Float X)
{
#if UE_PLATFORM_MATH_USE_SVML
	return _mm_atan_ps(X);
#else
	//return SSE::atan_ps(X);
	AlignedFloat4 Floats(X);
	Floats[0] = FMath::Atan(Floats[0]);
	Floats[1] = FMath::Atan(Floats[1]);
	Floats[2] = FMath::Atan(Floats[2]);
	Floats[3] = FMath::Atan(Floats[3]);
	return Floats.ToVectorRegister();
#endif
}

FORCEINLINE VectorRegister4Double VectorATan(VectorRegister4Double X)
{
#if UE_PLATFORM_MATH_USE_SVML_AVX
	return _mm256_atan_pd(X);
#elif UE_PLATFORM_MATH_USE_SVML
	return VectorRegister4Double(_mm_atan_pd(X.XY), _mm_atan_pd(X.ZW));
#else
	AlignedDouble4 Doubles(X);
	Doubles[0] = FMath::Atan(Doubles[0]);
	Doubles[1] = FMath::Atan(Doubles[1]);
	Doubles[2] = FMath::Atan(Doubles[2]);
	Doubles[3] = FMath::Atan(Doubles[3]);
	return Doubles.ToVectorRegister();
#endif
}

FORCEINLINE VectorRegister4Float VectorATan2(VectorRegister4Float Y, VectorRegister4Float X)
{
#if UE_PLATFORM_MATH_USE_SVML
	return _mm_atan2_ps(Y, X);
#else
	//return SSE::atan2_ps(Y, X);
	AlignedFloat4 FloatsY(Y);
	AlignedFloat4 FloatsX(X);
	FloatsY[0] = FMath::Atan2(FloatsY[0], FloatsX[0]);
	FloatsY[1] = FMath::Atan2(FloatsY[1], FloatsX[1]);
	FloatsY[2] = FMath::Atan2(FloatsY[2], FloatsX[2]);
	FloatsY[3] = FMath::Atan2(FloatsY[3], FloatsX[3]);
	return FloatsY.ToVectorRegister();
#endif
}

FORCEINLINE VectorRegister4Double VectorATan2(VectorRegister4Double Y, VectorRegister4Double X)
{
#if UE_PLATFORM_MATH_USE_SVML_AVX
	return _mm256_atan2_pd(Y, X);
#elif UE_PLATFORM_MATH_USE_SVML
	return VectorRegister4Double(_mm_atan2_pd(Y.XY, X.XY), _mm_atan2_pd(Y.ZW, X.ZW));
#else
	AlignedDouble4 DoublesY(Y);
	AlignedDouble4 DoublesX(X);
	DoublesY[0] = FMath::Atan2(DoublesY[0], DoublesX[0]);
	DoublesY[1] = FMath::Atan2(DoublesY[1], DoublesX[1]);
	DoublesY[2] = FMath::Atan2(DoublesY[2], DoublesX[2]);
	DoublesY[3] = FMath::Atan2(DoublesY[3], DoublesX[3]);
	return DoublesY.ToVectorRegister();
#endif
}


//////////////////////////////////////////////////////////////////////////
//Integer ops

//Bitwise
/** = a & b */
#define VectorIntAnd(A, B)		_mm_and_si128(A, B)
/** = a | b */
#define VectorIntOr(A, B)		_mm_or_si128(A, B)
/** = a ^ b */
#define VectorIntXor(A, B)		_mm_xor_si128(A, B)
/** = (~a) & b */
#define VectorIntAndNot(A, B)	_mm_andnot_si128(A, B)
/** = ~a */
#define VectorIntNot(A)	_mm_xor_si128(A, GlobalVectorConstants::IntAllMask)

//Comparison
#define VectorIntCompareEQ(A, B)	_mm_cmpeq_epi32(A,B)
#define VectorIntCompareNEQ(A, B)	VectorIntNot(_mm_cmpeq_epi32(A,B))
#define VectorIntCompareGT(A, B)	_mm_cmpgt_epi32(A,B)
#define VectorIntCompareLT(A, B)	_mm_cmplt_epi32(A,B)
#define VectorIntCompareGE(A, B)	VectorIntNot(VectorIntCompareLT(A,B))
#define VectorIntCompareLE(A, B)	VectorIntNot(VectorIntCompareGT(A,B))


FORCEINLINE VectorRegister4Int VectorIntSelect(VectorRegister4Int Mask, VectorRegister4Int Vec1, VectorRegister4Int Vec2)
{
	// Can't use PBLENDVB in general because this is a bitwise select, not byte-lane-wise
	return _mm_or_si128(_mm_and_si128(Mask, Vec1), _mm_andnot_si128(Mask, Vec2));
}

//Arithmetic
#define VectorIntAdd(A, B)	_mm_add_epi32(A, B)
#define VectorIntSubtract(A, B)	_mm_sub_epi32(A, B)

FORCEINLINE VectorRegister4Int VectorIntMultiply(VectorRegister4Int A, VectorRegister4Int B)
{
	return _mm_mullo_epi32(A, B);
}

#define VectorIntNegate(A) VectorIntSubtract(GlobalVectorConstants::IntZero, A)

FORCEINLINE VectorRegister4Int VectorIntMin(VectorRegister4Int A, VectorRegister4Int B)
{
	return _mm_min_epi32(A, B);
}

FORCEINLINE VectorRegister4Int VectorIntMax(VectorRegister4Int A, VectorRegister4Int B)
{
	return _mm_max_epi32(A, B);
}

FORCEINLINE VectorRegister4Int VectorIntAbs(VectorRegister4Int A)
{
	return _mm_abs_epi32(A);
}

FORCEINLINE VectorRegister4Int VectorIntClamp(VectorRegister4Int Vec1, VectorRegister4Int Vec2, VectorRegister4Int Vec3) 
{
	return VectorIntMin(VectorIntMax(Vec1, Vec2), Vec3);
}

#define VectorIntSign(A) VectorIntSelect(VectorIntCompareGE(A, GlobalVectorConstants::IntZero), GlobalVectorConstants::IntOne, GlobalVectorConstants::IntMinusOne)

#define VectorIntToFloat(A) _mm_cvtepi32_ps(A)

FORCEINLINE VectorRegister4Int VectorFloatToInt(VectorRegister4Float A)
{
	return _mm_cvttps_epi32(A);
}

// TODO: LWC: potential loss of data
FORCEINLINE VectorRegister4Int VectorFloatToInt(VectorRegister4Double A)
{
	return VectorFloatToInt(MakeVectorRegisterFloatFromDouble(A));
}

FORCEINLINE VectorRegister4Int VectorDoubleToInt(VectorRegister4Double Vec)
{
	VectorRegister4Int A = _mm_cvttpd_epi32(Vec.GetXY());
	VectorRegister4Int B = _mm_cvttpd_epi32(Vec.GetZW());
	return _mm_unpacklo_epi64(A, B);
}

FORCEINLINE VectorRegister4Int VectorShuffleByte4(VectorRegister4Int Vec, VectorRegister4Int Mask)
{
	return _mm_shuffle_epi8(Vec, Mask);
}


//Loads and stores

/**
* Stores a vector to memory (aligned or unaligned).
*
* @param Vec	Vector to store
* @param Ptr	Memory pointer
*/
#define VectorIntStore( Vec, Ptr )			_mm_storeu_si128( (VectorRegister4Int*)(Ptr), Vec )
#define VectorIntStore_16( Vec, Ptr )       _mm_storeu_si64( (VectorRegister4Int*)(Ptr), Vec )

/**
* Loads 4 int32s from unaligned memory.
*
* @param Ptr	Unaligned memory pointer to the 4 int32s
* @return		VectorRegister4Int(Ptr[0], Ptr[1], Ptr[2], Ptr[3])
*/
#define VectorIntLoad( Ptr )				_mm_loadu_si128( (VectorRegister4Int*)(Ptr) )
#define VectorIntLoad_16( Ptr )             _mm_loadu_si64 ( (VectorRegister4Int*)(Ptr) )
/**
* Stores a vector to memory (aligned).
*
* @param Vec	Vector to store
* @param Ptr	Aligned Memory pointer
*/
#define VectorIntStoreAligned( Vec, Ptr )			_mm_store_si128( (VectorRegister4Int*)(Ptr), Vec )

/**
* Loads 4 int32s from aligned memory.
*
* @param Ptr	Aligned memory pointer to the 4 int32s
* @return		VectorRegister4Int(Ptr[0], Ptr[1], Ptr[2], Ptr[3])
*/
#define VectorIntLoadAligned( Ptr )				_mm_load_si128( (VectorRegister4Int*)(Ptr) )

/**
* Loads 1 int32 from unaligned memory into all components of a vector register.
*
* @param Ptr	Unaligned memory pointer to the 4 int32s
* @return		VectorRegister4Int(*Ptr, *Ptr, *Ptr, *Ptr)
*/
#define VectorIntLoad1(Ptr)                         _mm_set1_epi32(*(Ptr))
#define VectorIntLoad1_16(Ptr)                      _mm_set1_epi16(*(Ptr))
#define VectorSetZero()								_mm_setzero_si128()
#define VectorSet1(F)								_mm_set1_ps(F)
#define VectorIntSet1(F)							_mm_set1_epi32(F)
#define VectorShiftLeftImm(Vec, ImmAmt)             _mm_slli_epi32(Vec, ImmAmt)
#define VectorShiftRightImmArithmetic(Vec, ImmAmt)  _mm_srai_epi32(Vec, ImmAmt)
#define VectorShiftRightImmLogical(Vec, ImmAmt)     _mm_srli_epi32(Vec, ImmAmt)
#define VectorCastIntToFloat(Vec)                   _mm_castsi128_ps(Vec)
#define VectorCastFloatToInt(Vec)                   _mm_castps_si128(Vec)
#define VectorCastDoubleToInt(Vec)                  _mm_castpd_si128(Vec)
#define VectorCastIntToDouble(Vec)                  _mm_castsi128_pd(Vec)
#define VectorShuffleImmediate(Vec, I0, I1, I2, I3) _mm_shuffle_epi32(Vec, _MM_SHUFFLE(I0, I1, I2, I3))
#define VectorIntExpandLow16To32(V0)				_mm_unpacklo_epi16(V0, _mm_setzero_si128())

#endif

// IWYU pragma: end_exports
