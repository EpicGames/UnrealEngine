// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Concepts/FloatingPoint.h"
#include "Concepts/Integral.h"
#include "Misc/AssertionMacros.h"
#include "HAL/PlatformMath.h"
#include "Math/MathFwd.h"
#include "Templates/Requires.h"

// Assert on non finite numbers. Used to track NaNs.
#ifndef ENABLE_NAN_DIAGNOSTIC
	#if UE_BUILD_DEBUG
		#define ENABLE_NAN_DIAGNOSTIC 1
	#else
		#define ENABLE_NAN_DIAGNOSTIC 0
	#endif
#endif

/*-----------------------------------------------------------------------------
	Definitions.
-----------------------------------------------------------------------------*/

// Forward declarations.
struct  FTwoVectors;
struct FLinearColor;
template<typename ElementType>
class TRange;

/*-----------------------------------------------------------------------------
	Floating point constants.
-----------------------------------------------------------------------------*/

// These macros can have different values across modules inside a single codebase.
// Tell the IncludeTool static analyzer to ignore that divergence in state:
#define UE_INCLUDETOOL_IGNORE_INCONSISTENT_STATE

// Define this to 1 in a module's .Build.cs to make legacy names issue deprecation warnings.
#ifndef UE_DEPRECATE_LEGACY_MATH_CONSTANT_MACRO_NAMES
	#define UE_DEPRECATE_LEGACY_MATH_CONSTANT_MACRO_NAMES 0
#endif

// Define this to 0 in a module's .Build.cs to stop that module recognizing the old identifiers.
#ifndef UE_DEFINE_LEGACY_MATH_CONSTANT_MACRO_NAMES
	#define UE_DEFINE_LEGACY_MATH_CONSTANT_MACRO_NAMES 1
#endif

// Process for fixing up a module's use of legacy names:
//
// - Define UE_DEPRECATE_LEGACY_MATH_CONSTANT_MACRO_NAMES=1 to the module's .Build.cs.
// - Build and fix all warnings.
// - Remove UE_DEPRECATE_LEGACY_MATH_CONSTANT_MACRO_NAMES=1 from the module's .Build.cs.
// - Define UE_DEFINE_LEGACY_MATH_CONSTANT_MACRO_NAMES=0 from the module's .Build.cs to stop new usage of the legacy macros.

#if UE_DEFINE_LEGACY_MATH_CONSTANT_MACRO_NAMES
	#if UE_DEPRECATE_LEGACY_MATH_CONSTANT_MACRO_NAMES
		#define UE_PRIVATE_MATH_DEPRECATION(Before, After) UE_DEPRECATED_MACRO(5.1, "The " #Before " macro has been deprecated in favor of " #After ".")
	#else
		#define UE_PRIVATE_MATH_DEPRECATION(Before, After)
	#endif

	#undef  PI
	#define PI										UE_PRIVATE_MATH_DEPRECATION(PI										, UE_PI										) UE_PI
	#define SMALL_NUMBER							UE_PRIVATE_MATH_DEPRECATION(SMALL_NUMBER							, UE_SMALL_NUMBER							) UE_SMALL_NUMBER
	#define KINDA_SMALL_NUMBER						UE_PRIVATE_MATH_DEPRECATION(KINDA_SMALL_NUMBER						, UE_KINDA_SMALL_NUMBER						) UE_KINDA_SMALL_NUMBER
	#define BIG_NUMBER								UE_PRIVATE_MATH_DEPRECATION(BIG_NUMBER								, UE_BIG_NUMBER								) UE_BIG_NUMBER
	#define EULERS_NUMBER							UE_PRIVATE_MATH_DEPRECATION(EULERS_NUMBER							, UE_EULERS_NUMBER							) UE_EULERS_NUMBER
	#define FLOAT_NON_FRACTIONAL					UE_PRIVATE_MATH_DEPRECATION(FLOAT_NON_FRACTIONAL					, UE_FLOAT_NON_FRACTIONAL					) UE_FLOAT_NON_FRACTIONAL
	#define DOUBLE_PI								UE_PRIVATE_MATH_DEPRECATION(DOUBLE_PI								, UE_DOUBLE_PI								) UE_DOUBLE_PI
	#define DOUBLE_SMALL_NUMBER						UE_PRIVATE_MATH_DEPRECATION(DOUBLE_SMALL_NUMBER						, UE_DOUBLE_SMALL_NUMBER					) UE_DOUBLE_SMALL_NUMBER
	#define DOUBLE_KINDA_SMALL_NUMBER				UE_PRIVATE_MATH_DEPRECATION(DOUBLE_KINDA_SMALL_NUMBER				, UE_DOUBLE_KINDA_SMALL_NUMBER				) UE_DOUBLE_KINDA_SMALL_NUMBER
	#define DOUBLE_BIG_NUMBER						UE_PRIVATE_MATH_DEPRECATION(DOUBLE_BIG_NUMBER						, UE_DOUBLE_BIG_NUMBER						) UE_DOUBLE_BIG_NUMBER
	#define DOUBLE_EULERS_NUMBER					UE_PRIVATE_MATH_DEPRECATION(DOUBLE_EULERS_NUMBER					, UE_DOUBLE_EULERS_NUMBER					) UE_DOUBLE_EULERS_NUMBER
	#define DOUBLE_UE_GOLDEN_RATIO					UE_PRIVATE_MATH_DEPRECATION(DOUBLE_UE_GOLDEN_RATIO					, UE_DOUBLE_GOLDEN_RATIO					) UE_DOUBLE_GOLDEN_RATIO
	#define DOUBLE_NON_FRACTIONAL					UE_PRIVATE_MATH_DEPRECATION(DOUBLE_NON_FRACTIONAL					, UE_DOUBLE_NON_FRACTIONAL					) UE_DOUBLE_NON_FRACTIONAL
	#define MAX_FLT									UE_PRIVATE_MATH_DEPRECATION(MAX_FLT									, UE_MAX_FLT								) UE_MAX_FLT
	#define INV_PI									UE_PRIVATE_MATH_DEPRECATION(INV_PI									, UE_INV_PI									) UE_INV_PI
	#define HALF_PI									UE_PRIVATE_MATH_DEPRECATION(HALF_PI									, UE_HALF_PI								) UE_HALF_PI
	#define TWO_PI									UE_PRIVATE_MATH_DEPRECATION(TWO_PI									, UE_TWO_PI									) UE_TWO_PI
	#define PI_SQUARED								UE_PRIVATE_MATH_DEPRECATION(PI_SQUARED								, UE_PI_SQUARED								) UE_PI_SQUARED
	#define DOUBLE_INV_PI							UE_PRIVATE_MATH_DEPRECATION(DOUBLE_INV_PI							, UE_DOUBLE_INV_PI							) UE_DOUBLE_INV_PI
	#define DOUBLE_HALF_PI							UE_PRIVATE_MATH_DEPRECATION(DOUBLE_HALF_PI							, UE_DOUBLE_HALF_PI							) UE_DOUBLE_HALF_PI
	#define DOUBLE_TWO_PI							UE_PRIVATE_MATH_DEPRECATION(DOUBLE_TWO_PI							, UE_DOUBLE_TWO_PI							) UE_DOUBLE_TWO_PI
	#define DOUBLE_PI_SQUARED						UE_PRIVATE_MATH_DEPRECATION(DOUBLE_PI_SQUARED						, UE_DOUBLE_PI_SQUARED						) UE_DOUBLE_PI_SQUARED
	#define DOUBLE_UE_SQRT_2						UE_PRIVATE_MATH_DEPRECATION(DOUBLE_UE_SQRT_2						, UE_DOUBLE_SQRT_2							) UE_DOUBLE_SQRT_2
	#define DOUBLE_UE_SQRT_3						UE_PRIVATE_MATH_DEPRECATION(DOUBLE_UE_SQRT_3						, UE_DOUBLE_SQRT_3							) UE_DOUBLE_SQRT_3
	#define DOUBLE_UE_INV_SQRT_2					UE_PRIVATE_MATH_DEPRECATION(DOUBLE_UE_INV_SQRT_2					, UE_DOUBLE_INV_SQRT_2						) UE_DOUBLE_INV_SQRT_2
	#define DOUBLE_UE_INV_SQRT_3					UE_PRIVATE_MATH_DEPRECATION(DOUBLE_UE_INV_SQRT_3					, UE_DOUBLE_INV_SQRT_3						) UE_DOUBLE_INV_SQRT_3
	#define DOUBLE_UE_HALF_SQRT_2					UE_PRIVATE_MATH_DEPRECATION(DOUBLE_UE_HALF_SQRT_2					, UE_DOUBLE_HALF_SQRT_2						) UE_DOUBLE_HALF_SQRT_2
	#define DOUBLE_UE_HALF_SQRT_3					UE_PRIVATE_MATH_DEPRECATION(DOUBLE_UE_HALF_SQRT_3					, UE_DOUBLE_HALF_SQRT_3						) UE_DOUBLE_HALF_SQRT_3
	#define DELTA									UE_PRIVATE_MATH_DEPRECATION(DELTA									, UE_DELTA									) UE_DELTA
	#define DOUBLE_DELTA							UE_PRIVATE_MATH_DEPRECATION(DOUBLE_DELTA							, UE_DOUBLE_DELTA							) UE_DOUBLE_DELTA
	#define FLOAT_NORMAL_THRESH						UE_PRIVATE_MATH_DEPRECATION(FLOAT_NORMAL_THRESH						, UE_FLOAT_NORMAL_THRESH					) UE_FLOAT_NORMAL_THRESH
	#define DOUBLE_NORMAL_THRESH					UE_PRIVATE_MATH_DEPRECATION(DOUBLE_NORMAL_THRESH					, UE_DOUBLE_NORMAL_THRESH					) UE_DOUBLE_NORMAL_THRESH
	#define THRESH_POINT_ON_PLANE					UE_PRIVATE_MATH_DEPRECATION(THRESH_POINT_ON_PLANE					, UE_THRESH_POINT_ON_PLANE					) UE_THRESH_POINT_ON_PLANE
	#define THRESH_POINT_ON_SIDE					UE_PRIVATE_MATH_DEPRECATION(THRESH_POINT_ON_SIDE					, UE_THRESH_POINT_ON_SIDE					) UE_THRESH_POINT_ON_SIDE
	#define THRESH_POINTS_ARE_SAME					UE_PRIVATE_MATH_DEPRECATION(THRESH_POINTS_ARE_SAME					, UE_THRESH_POINTS_ARE_SAME					) UE_THRESH_POINTS_ARE_SAME
	#define THRESH_POINTS_ARE_NEAR					UE_PRIVATE_MATH_DEPRECATION(THRESH_POINTS_ARE_NEAR					, UE_THRESH_POINTS_ARE_NEAR					) UE_THRESH_POINTS_ARE_NEAR
	#define THRESH_NORMALS_ARE_SAME					UE_PRIVATE_MATH_DEPRECATION(THRESH_NORMALS_ARE_SAME					, UE_THRESH_NORMALS_ARE_SAME				) UE_THRESH_NORMALS_ARE_SAME
	#define THRESH_UVS_ARE_SAME						UE_PRIVATE_MATH_DEPRECATION(THRESH_UVS_ARE_SAME						, UE_THRESH_UVS_ARE_SAME					) UE_THRESH_UVS_ARE_SAME
	#define THRESH_VECTORS_ARE_NEAR					UE_PRIVATE_MATH_DEPRECATION(THRESH_VECTORS_ARE_NEAR					, UE_THRESH_VECTORS_ARE_NEAR				) UE_THRESH_VECTORS_ARE_NEAR
	#define THRESH_SPLIT_POLY_WITH_PLANE			UE_PRIVATE_MATH_DEPRECATION(THRESH_SPLIT_POLY_WITH_PLANE			, UE_THRESH_SPLIT_POLY_WITH_PLANE			) UE_THRESH_SPLIT_POLY_WITH_PLANE
	#define THRESH_SPLIT_POLY_PRECISELY				UE_PRIVATE_MATH_DEPRECATION(THRESH_SPLIT_POLY_PRECISELY				, UE_THRESH_SPLIT_POLY_PRECISELY			) UE_THRESH_SPLIT_POLY_PRECISELY
	#define THRESH_ZERO_NORM_SQUARED				UE_PRIVATE_MATH_DEPRECATION(THRESH_ZERO_NORM_SQUARED				, UE_THRESH_ZERO_NORM_SQUARED				) UE_THRESH_ZERO_NORM_SQUARED
	#define THRESH_NORMALS_ARE_PARALLEL				UE_PRIVATE_MATH_DEPRECATION(THRESH_NORMALS_ARE_PARALLEL				, UE_THRESH_NORMALS_ARE_PARALLEL			) UE_THRESH_NORMALS_ARE_PARALLEL
	#define THRESH_NORMALS_ARE_ORTHOGONAL			UE_PRIVATE_MATH_DEPRECATION(THRESH_NORMALS_ARE_ORTHOGONAL			, UE_THRESH_NORMALS_ARE_ORTHOGONAL			) UE_THRESH_NORMALS_ARE_ORTHOGONAL
	#define THRESH_VECTOR_NORMALIZED				UE_PRIVATE_MATH_DEPRECATION(THRESH_VECTOR_NORMALIZED				, UE_THRESH_VECTOR_NORMALIZED				) UE_THRESH_VECTOR_NORMALIZED
	#define THRESH_QUAT_NORMALIZED					UE_PRIVATE_MATH_DEPRECATION(THRESH_QUAT_NORMALIZED					, UE_THRESH_QUAT_NORMALIZED					) UE_THRESH_QUAT_NORMALIZED
	#define DOUBLE_THRESH_POINT_ON_PLANE			UE_PRIVATE_MATH_DEPRECATION(DOUBLE_THRESH_POINT_ON_PLANE			, UE_DOUBLE_THRESH_POINT_ON_PLANE			) UE_DOUBLE_THRESH_POINT_ON_PLANE
	#define DOUBLE_THRESH_POINT_ON_SIDE				UE_PRIVATE_MATH_DEPRECATION(DOUBLE_THRESH_POINT_ON_SIDE				, UE_DOUBLE_THRESH_POINT_ON_SIDE			) UE_DOUBLE_THRESH_POINT_ON_SIDE
	#define DOUBLE_THRESH_POINTS_ARE_SAME			UE_PRIVATE_MATH_DEPRECATION(DOUBLE_THRESH_POINTS_ARE_SAME			, UE_DOUBLE_THRESH_POINTS_ARE_SAME			) UE_DOUBLE_THRESH_POINTS_ARE_SAME
	#define DOUBLE_THRESH_POINTS_ARE_NEAR			UE_PRIVATE_MATH_DEPRECATION(DOUBLE_THRESH_POINTS_ARE_NEAR			, UE_DOUBLE_THRESH_POINTS_ARE_NEAR			) UE_DOUBLE_THRESH_POINTS_ARE_NEAR
	#define DOUBLE_THRESH_NORMALS_ARE_SAME			UE_PRIVATE_MATH_DEPRECATION(DOUBLE_THRESH_NORMALS_ARE_SAME			, UE_DOUBLE_THRESH_NORMALS_ARE_SAME			) UE_DOUBLE_THRESH_NORMALS_ARE_SAME
	#define DOUBLE_THRESH_UVS_ARE_SAME				UE_PRIVATE_MATH_DEPRECATION(DOUBLE_THRESH_UVS_ARE_SAME				, UE_DOUBLE_THRESH_UVS_ARE_SAME				) UE_DOUBLE_THRESH_UVS_ARE_SAME
	#define DOUBLE_THRESH_VECTORS_ARE_NEAR			UE_PRIVATE_MATH_DEPRECATION(DOUBLE_THRESH_VECTORS_ARE_NEAR			, UE_DOUBLE_THRESH_VECTORS_ARE_NEAR			) UE_DOUBLE_THRESH_VECTORS_ARE_NEAR
	#define DOUBLE_THRESH_SPLIT_POLY_WITH_PLANE		UE_PRIVATE_MATH_DEPRECATION(DOUBLE_THRESH_SPLIT_POLY_WITH_PLANE		, UE_DOUBLE_THRESH_SPLIT_POLY_WITH_PLANE	) UE_DOUBLE_THRESH_SPLIT_POLY_WITH_PLANE
	#define DOUBLE_THRESH_SPLIT_POLY_PRECISELY		UE_PRIVATE_MATH_DEPRECATION(DOUBLE_THRESH_SPLIT_POLY_PRECISELY		, UE_DOUBLE_THRESH_SPLIT_POLY_PRECISELY		) UE_DOUBLE_THRESH_SPLIT_POLY_PRECISELY
	#define DOUBLE_THRESH_ZERO_NORM_SQUARED			UE_PRIVATE_MATH_DEPRECATION(DOUBLE_THRESH_ZERO_NORM_SQUARED			, UE_DOUBLE_THRESH_ZERO_NORM_SQUARED		) UE_DOUBLE_THRESH_ZERO_NORM_SQUARED
	#define DOUBLE_THRESH_NORMALS_ARE_PARALLEL		UE_PRIVATE_MATH_DEPRECATION(DOUBLE_THRESH_NORMALS_ARE_PARALLEL		, UE_DOUBLE_THRESH_NORMALS_ARE_PARALLEL		) UE_DOUBLE_THRESH_NORMALS_ARE_PARALLEL
	#define DOUBLE_THRESH_NORMALS_ARE_ORTHOGONAL	UE_PRIVATE_MATH_DEPRECATION(DOUBLE_THRESH_NORMALS_ARE_ORTHOGONAL	, UE_DOUBLE_THRESH_NORMALS_ARE_ORTHOGONAL	) UE_DOUBLE_THRESH_NORMALS_ARE_ORTHOGONAL
	#define DOUBLE_THRESH_VECTOR_NORMALIZED			UE_PRIVATE_MATH_DEPRECATION(DOUBLE_THRESH_VECTOR_NORMALIZED			, UE_DOUBLE_THRESH_VECTOR_NORMALIZED		) UE_DOUBLE_THRESH_VECTOR_NORMALIZED
	#define DOUBLE_THRESH_QUAT_NORMALIZED			UE_PRIVATE_MATH_DEPRECATION(DOUBLE_THRESH_QUAT_NORMALIZED			, UE_DOUBLE_THRESH_QUAT_NORMALIZED			) UE_DOUBLE_THRESH_QUAT_NORMALIZED
#endif

#undef UE_INCLUDETOOL_IGNORE_INCONSISTENT_STATE

#define UE_PI 					(3.1415926535897932f)	/* Extra digits if needed: 3.1415926535897932384626433832795f */
#define UE_SMALL_NUMBER			(1.e-8f)
#define UE_KINDA_SMALL_NUMBER	(1.e-4f)
#define UE_BIG_NUMBER			(3.4e+38f)
#define UE_EULERS_NUMBER		(2.71828182845904523536f)
#define UE_GOLDEN_RATIO			(1.6180339887498948482045868343656381f)	/* Also known as divine proportion, golden mean, or golden section - related to the Fibonacci Sequence = (1 + sqrt(5)) / 2 */
#define UE_FLOAT_NON_FRACTIONAL (8388608.f) /* All single-precision floating point numbers greater than or equal to this have no fractional value. */


#define UE_DOUBLE_PI					(3.141592653589793238462643383279502884197169399)
#define UE_DOUBLE_SMALL_NUMBER			(1.e-8)
#define UE_DOUBLE_KINDA_SMALL_NUMBER	(1.e-4)
#define UE_DOUBLE_BIG_NUMBER			(3.4e+38)
#define UE_DOUBLE_EULERS_NUMBER			(2.7182818284590452353602874713526624977572)
#define UE_DOUBLE_GOLDEN_RATIO			(1.6180339887498948482045868343656381)	/* Also known as divine proportion, golden mean, or golden section - related to the Fibonacci Sequence = (1 + sqrt(5)) / 2 */
#define UE_DOUBLE_NON_FRACTIONAL		(4503599627370496.0) /* All double-precision floating point numbers greater than or equal to this have no fractional value. 2^52 */

// Copied from float.h
#define UE_MAX_FLT 3.402823466e+38F

// Aux constants.
#define UE_INV_PI			(0.31830988618f)
#define UE_HALF_PI			(1.57079632679f)
#define UE_TWO_PI			(6.28318530717f)
#define UE_PI_SQUARED		(9.86960440108f)

#define UE_DOUBLE_INV_PI		(0.31830988618379067154)
#define UE_DOUBLE_HALF_PI		(1.57079632679489661923)
#define UE_DOUBLE_TWO_PI		(6.28318530717958647692)
#define UE_DOUBLE_PI_SQUARED	(9.86960440108935861883)

#define UE_LN2		(0.69314718056f)

// Common square roots
#define UE_SQRT_2		(1.4142135623730950488016887242097f)
#define UE_SQRT_3		(1.7320508075688772935274463415059f)
#define UE_INV_SQRT_2	(0.70710678118654752440084436210485f)
#define UE_INV_SQRT_3	(0.57735026918962576450914878050196f)
#define UE_HALF_SQRT_2	(0.70710678118654752440084436210485f)
#define UE_HALF_SQRT_3	(0.86602540378443864676372317075294f)

#define UE_DOUBLE_SQRT_2		(1.4142135623730950488016887242097)
#define UE_DOUBLE_SQRT_3		(1.7320508075688772935274463415059)
#define UE_DOUBLE_INV_SQRT_2	(0.70710678118654752440084436210485)
#define UE_DOUBLE_INV_SQRT_3	(0.57735026918962576450914878050196)
#define UE_DOUBLE_HALF_SQRT_2	(0.70710678118654752440084436210485)
#define UE_DOUBLE_HALF_SQRT_3	(0.86602540378443864676372317075294)

// Common metric unit conversion
#define UE_KM_TO_M   (1000.f)
#define UE_M_TO_KM   (0.001f)
#define UE_CM_TO_M   (0.01f)
#define UE_M_TO_CM   (100.f)
#define UE_CM2_TO_M2 (0.0001f)
#define UE_M2_TO_CM2 (10000.f)

// Magic numbers for numerical precision.
#define UE_DELTA		(0.00001f)
#define UE_DOUBLE_DELTA	(0.00001 )

/**
 * Lengths of normalized vectors (These are half their maximum values
 * to assure that dot products with normalized vectors don't overflow).
 */
#define UE_FLOAT_NORMAL_THRESH			(0.0001f)
#define UE_DOUBLE_NORMAL_THRESH			(0.0001)

//
// Magic numbers for numerical precision.
//
#define UE_THRESH_POINT_ON_PLANE				(0.10f)		/* Thickness of plane for front/back/inside test */
#define UE_THRESH_POINT_ON_SIDE					(0.20f)		/* Thickness of polygon side's side-plane for point-inside/outside/on side test */
#define UE_THRESH_POINTS_ARE_SAME				(0.00002f)	/* Two points are same if within this distance */
#define UE_THRESH_POINTS_ARE_NEAR				(0.015f)	/* Two points are near if within this distance and can be combined if imprecise math is ok */
#define UE_THRESH_NORMALS_ARE_SAME				(0.00002f)	/* Two normal points are same if within this distance */
#define UE_THRESH_UVS_ARE_SAME					(0.0009765625f)/* Two UV are same if within this threshold (1.0f/1024f) */
															/* Making this too large results in incorrect CSG classification and disaster */
#define UE_THRESH_VECTORS_ARE_NEAR				(0.0004f)	/* Two vectors are near if within this distance and can be combined if imprecise math is ok */
															/* Making this too large results in lighting problems due to inaccurate texture coordinates */
#define UE_THRESH_SPLIT_POLY_WITH_PLANE			(0.25f)		/* A plane splits a polygon in half */
#define UE_THRESH_SPLIT_POLY_PRECISELY			(0.01f)		/* A plane exactly splits a polygon */
#define UE_THRESH_ZERO_NORM_SQUARED				(0.0001f)	/* Size of a unit normal that is considered "zero", squared */
#define UE_THRESH_NORMALS_ARE_PARALLEL			(0.999845f)	/* Two unit vectors are parallel if abs(A dot B) is greater than or equal to this. This is roughly cosine(1.0 degrees). */
#define UE_THRESH_NORMALS_ARE_ORTHOGONAL		(0.017455f)	/* Two unit vectors are orthogonal (perpendicular) if abs(A dot B) is less than or equal this. This is roughly cosine(89.0 degrees). */

#define UE_THRESH_VECTOR_NORMALIZED				(0.01f)		/** Allowed error for a normalized vector (against squared magnitude) */
#define UE_THRESH_QUAT_NORMALIZED				(0.01f)		/** Allowed error for a normalized quaternion (against squared magnitude) */

// Double precision values
#define UE_DOUBLE_THRESH_POINT_ON_PLANE			(0.10)		/* Thickness of plane for front/back/inside test */
#define UE_DOUBLE_THRESH_POINT_ON_SIDE			(0.20)		/* Thickness of polygon side's side-plane for point-inside/outside/on side test */
#define UE_DOUBLE_THRESH_POINTS_ARE_SAME		(0.00002)	/* Two points are same if within this distance */
#define UE_DOUBLE_THRESH_POINTS_ARE_NEAR		(0.015)		/* Two points are near if within this distance and can be combined if imprecise math is ok */
#define UE_DOUBLE_THRESH_NORMALS_ARE_SAME		(0.00002)	/* Two normal points are same if within this distance */
#define UE_DOUBLE_THRESH_UVS_ARE_SAME			(0.0009765625)/* Two UV are same if within this threshold (1.0/1024.0) */
															/* Making this too large results in incorrect CSG classification and disaster */
#define UE_DOUBLE_THRESH_VECTORS_ARE_NEAR		(0.0004)	/* Two vectors are near if within this distance and can be combined if imprecise math is ok */
															/* Making this too large results in lighting problems due to inaccurate texture coordinates */
#define UE_DOUBLE_THRESH_SPLIT_POLY_WITH_PLANE	(0.25)		/* A plane splits a polygon in half */
#define UE_DOUBLE_THRESH_SPLIT_POLY_PRECISELY	(0.01)		/* A plane exactly splits a polygon */
#define UE_DOUBLE_THRESH_ZERO_NORM_SQUARED		(0.0001)	/* Size of a unit normal that is considered "zero", squared */
#define UE_DOUBLE_THRESH_NORMALS_ARE_PARALLEL	(0.999845)	/* Two unit vectors are parallel if abs(A dot B) is greater than or equal to this. This is roughly cosine(1.0 degrees). */
#define UE_DOUBLE_THRESH_NORMALS_ARE_ORTHOGONAL	(0.017455)	/* Two unit vectors are orthogonal (perpendicular) if abs(A dot B) is less than or equal this. This is roughly cosine(89.0 degrees). */

#define UE_DOUBLE_THRESH_VECTOR_NORMALIZED		(0.01)		/** Allowed error for a normalized vector (against squared magnitude) */
#define UE_DOUBLE_THRESH_QUAT_NORMALIZED		(0.01)		/** Allowed error for a normalized quaternion (against squared magnitude) */

/*-----------------------------------------------------------------------------
	Global functions.
-----------------------------------------------------------------------------*/

/**
 * Template helper for FMath::Lerp<>() and related functions.
 * By default, any type T is assumed to not need a custom Lerp implementation (Value=false).
 * However a class that requires custom functionality (eg FQuat) can specialize the template to define Value=true and
 * implement the Lerp() function and other similar functions and provide a custom implementation.
 * Example:
 * 
 *	template<> struct TCustomLerp< MyClass >
 *	{
 *		// Required to use our custom Lerp() function below.
 *		enum { Value = true };
 *
 *		// Implements for float Alpha param. You could also add overrides or make it a template param.
 *		static inline MyClass Lerp(const MyClass& A, const MyClass& B, const float& Alpha)
 *		{
 *			return MyClass::Lerp(A, B, Alpha); // Or do the computation here directly.
 *		}
 *	};
 */
template <typename T>
struct TCustomLerp
{
	constexpr static bool Value = false;
};

/**
 * Structure for all math helper functions, inherits from platform math to pick up platform-specific implementations
 * Check GenericPlatformMath.h for additional math functions
 */
struct FMath : public FPlatformMath
{
	// Random Number Functions

	/** Helper function for rand implementations. Returns a random number in [0..A) */
	[[nodiscard]] static UE_FORCEINLINE_HINT int32 RandHelper(int32 A)
	{
		// Note that on some platforms RAND_MAX is a large number so we cannot do ((rand()/(RAND_MAX+1)) * A)
		// or else we may include the upper bound results, which should be excluded.
		return A > 0 ? Min(TruncToInt(FRand() * (float)A), A - 1) : 0;
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT int64 RandHelper64(int64 A)
	{
		// Note that on some platforms RAND_MAX is a large number so we cannot do ((rand()/(RAND_MAX+1)) * A)
		// or else we may include the upper bound results, which should be excluded.
		return A > 0 ? Min<int64>(TruncToInt(FRand() * (float)A), A - 1) : 0;
	}

	/** Helper function for rand implementations. Returns a random number >= Min and <= Max */
	[[nodiscard]] static inline int32 RandRange(int32 Min, int32 Max)
	{
		const int32 Range = (Max - Min) + 1;
		return Min + RandHelper(Range);
	}

	[[nodiscard]] static inline int64 RandRange(int64 Min, int64 Max)
	{
		const int64 Range = (Max - Min) + 1;
		return Min + RandHelper64(Range);
	}

	/** Util to generate a random number in a range. Overloaded to distinguish from int32 version, where passing a float is typically a mistake. */
	[[nodiscard]] static UE_FORCEINLINE_HINT float RandRange(float InMin, float InMax)
	{
		return FRandRange(InMin, InMax);
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT double RandRange(double InMin, double InMax)
	{
		return FRandRange(InMin, InMax);
	}

	/** Util to generate a random number in a range. */
	[[nodiscard]] static UE_FORCEINLINE_HINT float FRandRange(float InMin, float InMax)
	{
		return InMin + (InMax - InMin) * FRand();
	}

	/** Util to generate a random number in a range. */
	[[nodiscard]] static UE_FORCEINLINE_HINT double FRandRange(double InMin, double InMax)
	{
		return InMin + (InMax - InMin) * FRand();	// LWC_TODO: Implement FRandDbl() for increased precision
	}

	RESOLVE_FLOAT_AMBIGUITY_2_ARGS(FRandRange);

	/** Util to generate a random boolean. */
	[[nodiscard]] static UE_FORCEINLINE_HINT bool RandBool()
	{
		return (RandRange(0,1) == 1) ? true : false;
	}

	/** Return a uniformly distributed random unit length vector = point on the unit sphere surface. */
	[[nodiscard]] static FVector VRand();
	
	/**
	 * Returns a random unit vector, uniformly distributed, within the specified cone
	 * ConeHalfAngleRad is the half-angle of cone, in radians.  Returns a normalized vector. 
	 */
	[[nodiscard]] static CORE_API FVector VRandCone(FVector const& Dir, float ConeHalfAngleRad);

	/** 
	 * This is a version of VRandCone that handles "squished" cones, i.e. with different angle limits in the Y and Z axes.
	 * Assumes world Y and Z, although this could be extended to handle arbitrary rotations.
	 */
	[[nodiscard]] static CORE_API FVector VRandCone(FVector const& Dir, float HorizontalConeHalfAngleRad, float VerticalConeHalfAngleRad);

	/** Returns a random point, uniformly distributed, within the specified radius */
	[[nodiscard]] static CORE_API FVector2D RandPointInCircle(float CircleRadius);

	/** Returns a random point within the passed in bounding box */
	[[nodiscard]] static CORE_API FVector RandPointInBox(const FBox& Box);

	/** 
	 * Given a direction vector and a surface normal, returns the vector reflected across the surface normal.
	 * Produces a result like shining a laser at a mirror!
	 *
	 * @param Direction Direction vector the ray is coming from.
	 * @param SurfaceNormal A normal of the surface the ray should be reflected on.
	 *
	 * @returns Reflected vector.
	 */
	[[nodiscard]] static CORE_API FVector GetReflectionVector(const FVector& Direction, const FVector& SurfaceNormal);
	
	// Predicates

	/** Checks if value is within a range, exclusive on MaxValue) */
	template< class T, class U> 
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT bool IsWithin(const T& TestValue, const U& MinValue, const U& MaxValue)
	{
		return ((TestValue >= MinValue) && (TestValue < MaxValue));
	}


	/** Checks if value is within a range, inclusive on MaxValue) */
	template< class T, class U> 
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT bool IsWithinInclusive(const T& TestValue, const U& MinValue, const U& MaxValue)
	{
		return ((TestValue>=MinValue) && (TestValue <= MaxValue));
	}
	
	/**
	 *	Checks if two floating point numbers are nearly equal.
	 *	@param A				First number to compare
	 *	@param B				Second number to compare
	 *	@param ErrorTolerance	Maximum allowed difference for considering them as 'nearly equal'
	 *	@return					true if A and B are nearly equal
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT bool IsNearlyEqual(float A, float B, float ErrorTolerance = UE_SMALL_NUMBER)
	{
		return Abs<float>( A - B ) <= ErrorTolerance;
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT bool IsNearlyEqual(double A, double B, double ErrorTolerance = UE_DOUBLE_SMALL_NUMBER)
	{
		return Abs<double>(A - B) <= ErrorTolerance;
	}

	RESOLVE_FLOAT_PREDICATE_AMBIGUITY_2_ARGS(IsNearlyEqual);
	RESOLVE_FLOAT_PREDICATE_AMBIGUITY_3_ARGS(IsNearlyEqual);

	/**
	 *	Checks if a floating point number is nearly zero.
	 *	@param Value			Number to compare
	 *	@param ErrorTolerance	Maximum allowed difference for considering Value as 'nearly zero'
	 *	@return					true if Value is nearly zero
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT bool IsNearlyZero(float Value, float ErrorTolerance = UE_SMALL_NUMBER)
	{
		return Abs<float>( Value ) <= ErrorTolerance;
	}

	/**
	 *	Checks if a floating point number is nearly zero.
	 *	@param Value			Number to compare
	 *	@param ErrorTolerance	Maximum allowed difference for considering Value as 'nearly zero'
	 *	@return					true if Value is nearly zero
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT bool IsNearlyZero(double Value, double ErrorTolerance = UE_DOUBLE_SMALL_NUMBER)
	{
		return Abs<double>( Value ) <= ErrorTolerance;
	}

	RESOLVE_FLOAT_PREDICATE_AMBIGUITY_2_ARGS(IsNearlyZero);

private:
	template<typename FloatType, typename IntegralType, IntegralType SignedBit>
	static inline bool TIsNearlyEqualByULP(FloatType A, FloatType B, int32 MaxUlps)
	{
		// Any comparison with NaN always fails.
		if (FMath::IsNaN(A) || FMath::IsNaN(B))
		{
			return false;
		}

		// If either number is infinite, then ignore ULP and do a simple equality test. 
		// The rationale being that two infinities, of the same sign, should compare the same 
		// no matter the ULP, but FLT_MAX and Inf should not, even if they're neighbors in
		// their bit representation.
		if (!FMath::IsFinite(A) || !FMath::IsFinite(B))
		{
			return A == B;
		}

		// Convert the integer representation of the float from sign + magnitude to
		// a signed number representation where 0 is 1 << 31. This allows us to compare
		// ULP differences around zero values.
		auto FloatToSignedNumber = [](IntegralType V) {
			if (V & SignedBit)
			{
				return ~V + 1;
			}
			else
			{
				return SignedBit | V;
			}
		};

		IntegralType SNA = FloatToSignedNumber(FMath::AsUInt(A));
		IntegralType SNB = FloatToSignedNumber(FMath::AsUInt(B));
		IntegralType Distance = (SNA >= SNB) ? (SNA - SNB) : (SNB - SNA);
		return Distance <= IntegralType(MaxUlps);
	}

public:

	/**
	 *	Check if two floating point numbers are nearly equal to within specific number of 
	 *	units of last place (ULP). A single ULP difference between two floating point numbers
	 *	means that they have an adjacent representation and that no other floating point number
	 *	can be constructed to fit between them. This enables making consistent comparisons 
	 *	based on representational distance between floating point numbers, regardless of 
	 *	their magnitude. 
	 *
	 *	Use when the two numbers vary greatly in range. Otherwise, if absolute tolerance is
	 *	required, use IsNearlyEqual instead.
	 *  
	 *	Note: Since IEEE 754 floating point operations are guaranteed to be exact to 0.5 ULP,
	 *	a value of 4 ought to be sufficient for all but the most complex float operations.
	 * 
	 *	@param A				First number to compare
	 *	@param B				Second number to compare
	 *	@param MaxUlps          The maximum ULP distance by which neighboring floating point 
	 *	                        numbers are allowed to differ.
	 *	@return					true if the two values are nearly equal.
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT bool IsNearlyEqualByULP(float A, float B, int32 MaxUlps = 4)
	{
		return TIsNearlyEqualByULP<float, uint32, uint32(1U << 31)>(A, B, MaxUlps);
	}

	/**
	 *	Check if two floating point numbers are nearly equal to within specific number of
	 *	units of last place (ULP). A single ULP difference between two floating point numbers
	 *	means that they have an adjacent representation and that no other floating point number
	 *	can be constructed to fit between them. This enables making consistent comparisons
	 *	based on representational distance between floating point numbers, regardless of
	 *	their magnitude.
	 *
	 *	Note: Since IEEE 754 floating point operations are guaranteed to be exact to 0.5 ULP,
	 *	a value of 4 ought to be sufficient for all but the most complex float operations.
	 *
	 *	@param A				First number to compare
	 *	@param B				Second number to compare
	 *	@param MaxUlps          The maximum ULP distance by which neighboring floating point
	 *	                        numbers are allowed to differ.
	 *	@return					true if the two values are nearly equal.
	 */
	[[nodiscard]] static UE_FORCEINLINE_HINT bool IsNearlyEqualByULP(double A, double B, int32 MaxUlps = 4)
	{
		return TIsNearlyEqualByULP<double, uint64, uint64(1ULL << 63)>(A, B, MaxUlps);
	}

	/**
	*	Checks whether a number is a power of two.
	*	@param Value	Number to check
	*	@return			true if Value is a power of two
	*/
	template <typename T>
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT bool IsPowerOfTwo( T Value )
	{
		return ((Value & (Value - 1)) == (T)0);
	}

	/** Converts a float to a nearest less or equal integer. */
	[[nodiscard]] static UE_FORCEINLINE_HINT float Floor(float F)
	{
		return FloorToFloat(F);
	}

	/** Converts a double to a nearest less or equal integer. */
	[[nodiscard]] static UE_FORCEINLINE_HINT double Floor(double F)
	{
		return FloorToDouble(F);
	}

	/**
	 * Converts an integral type to a nearest less or equal integer.
	 * Unlike std::floor, it returns an IntegralType.
	 */
	template <UE::CIntegral IntegralType>
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT IntegralType Floor(IntegralType I)
	{
		return I;
	}


	// Math Operations

	/** Returns highest of 3 values */
	template< class T > 
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT T Max3( const T A, const T B, const T C )
	{
		return Max ( Max( A, B ), C );
	}

	/** Returns lowest of 3 values */
	template< class T > 
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT T Min3( const T A, const T B, const T C )
	{
		return Min ( Min( A, B ), C );
	}

	template< class T > 
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT int32 Max3Index( const T A, const T B, const T C )
	{
		return ( A > B ) ? ( ( A > C ) ? 0 : 2 ) : ( ( B > C ) ? 1 : 2 );
	}

	/** Returns index of the lowest value */
	template< class T > 
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT int32 Min3Index( const T A, const T B, const T C )
	{
		return ( A < B ) ? ( ( A < C ) ? 0 : 2 ) : ( ( B < C ) ? 1 : 2 );
	}

	/** Multiples value by itself */
	template< class T > 
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT T Square( const T A )
	{
		return A*A;
	}

	/** Cubes the value */
	template< class T > 
    [[nodiscard]] static constexpr UE_FORCEINLINE_HINT T Cube( const T A )
	{
		return A*A*A;
	}

	/** Clamps X to be between Min and Max, inclusive */
	template< class T >
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT T Clamp(const T X, const T MinValue, const T MaxValue)
	{
		return Max(Min(X, MaxValue), MinValue);
	}
	/** Allow mixing float/double arguments, promoting to highest precision type. */
	MIX_FLOATS_3_ARGS(Clamp);
	
	/** Clamps X to be between Min and Max, inclusive. Explicitly defined here for floats/doubles because static analysis gets confused between template and int versions. */
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT float Clamp(const float X, const float Min, const float Max) { return Clamp<float>(X, Min, Max); }
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT double Clamp(const double X, const double Min, const double Max) { return Clamp<double>(X, Min, Max); }

	/** Clamps X to be between Min and Max, inclusive. Overload to support mixed int64/int32 types. */
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT int64 Clamp(const int64 X, const int32 Min, const int32 Max) { return Clamp<int64>(X, Min, Max); }

public:
	/** Wraps X to be between Min and Max, inclusive. */
	/** When X can wrap to both Min and Max, it will wrap to Min if it lies below the range and wrap to Max if it is above the range. */
	template <UE::CFloatingPoint T>
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT T Wrap(const T X, const T Min, const T Max)
	{
		// Our asserts are not constexpr-friendly yet
		// checkSlow(Min <= Max);

		T Size = Max - Min;
		if (Size == 0)
		{
			// Guard against zero-sized ranges causing division by zero.
			return Max;
		}

		T EndVal = X;
		if (EndVal < Min)
		{
			T Mod = FMath::Modulo(Min - EndVal, Size);
			EndVal = (Mod != (T)0) ? Max - Mod : Min;
		}
		else if (EndVal > Max)
		{
			T Mod = FMath::Modulo(EndVal - Max, Size);
			EndVal = (Mod != (T)0) ? Min + Mod : Max;
		}
		return EndVal;
	}

	/** Wraps X to be between Min and Max, exclusive. Will never return Max. */
	template <UE::CIntegral T>
	[[nodiscard]] static constexpr inline T WrapExclusive(const T X, const T Min, const T Max)
	{
		// Use unsigned type allow for large ranges which don't overflow on subtraction.
		using SizeType = std::make_unsigned_t<T>;

		// Our asserts are not constexpr-friendly yet
		// checkSlow(Min <= Max);

		SizeType Size = (SizeType)((SizeType)Max - (SizeType)Min);
		if (Size == 0)
		{
			// Guard against zero-sized ranges causing division by zero.
			return Max;
		}

		SizeType Mod;
		if (X < Min)
		{
			Mod = (SizeType)FMath::Modulo((SizeType)((SizeType)Min - (SizeType)X), Size);
			if (Mod > 0)
			{
				Mod = (SizeType)(Size - Mod);
			}
		}
		else
		{
			Mod = (SizeType)FMath::Modulo((SizeType)((SizeType)X - (SizeType)Min), Size);
		}

		return (T)(Min + Mod);
	}

	template <typename ValueType, typename BaseType>
	[[nodiscard]] static constexpr inline auto Modulo(ValueType Value, BaseType Base)
	{
		if constexpr (std::is_floating_point_v<ValueType>)
		{
			return FMath::Fmod(Value, Base);
		}
		else
		{
			return Value % Base;
		}
	}

	/** Snaps a value to the nearest grid multiple */
	template< class T >
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT T GridSnap(T Location, T Grid)
	{
		return (Grid == T{}) ? Location : (Floor((Location + (Grid/(T)2)) / Grid) * Grid);
	}
	/** Allow mixing float/double arguments, promoting to highest precision type. */
	MIX_FLOATS_2_ARGS(GridSnap);

	/** Divides two integers and rounds up */
	template <class T>
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT T DivideAndRoundUp(T Dividend, T Divisor)
	{
		return (Dividend + Divisor - 1) / Divisor;
	}

	/** Divides two integers and rounds down */
	template <class T>
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT T DivideAndRoundDown(T Dividend, T Divisor)
	{
		return Dividend / Divisor;
	}

	/** Divides two integers and rounds to nearest */
	template <class T>
	[[nodiscard]] static constexpr inline T DivideAndRoundNearest(T Dividend, T Divisor)
	{
		return (Dividend >= 0)
			? (Dividend + Divisor / 2) / Divisor
			: (Dividend - Divisor / 2 + 1) / Divisor;
	}

	/** Performs a safe division, which returns the largest possible value when divisor is nearly zero */
	template <class T, class U> 
	[[nodiscard]] static constexpr auto SafeDivide(T Dividend, U Divisor) -> decltype(Dividend / Divisor)
	{
		return FMath::IsNearlyZero(Divisor) ? TNumericLimits<decltype(Dividend / Divisor)>::Max() : (Dividend / Divisor);
	};

	/**
	 * Computes the base 2 logarithm of the specified value
	 *
	 * @param Value the value to perform the log on
	 *
	 * @return the base 2 log of the value
	 */
	[[nodiscard]] static inline float Log2(float Value)
	{
		// Cached value for fast conversions
		constexpr float LogToLog2 = 1.44269502f; // 1.f / Loge(2.f)
		// Do the platform specific log and convert using the cached value
		return Loge(Value) * LogToLog2;
	}

	/**
	 * Computes the base 2 logarithm of the specified value
	 *
	 * @param Value the value to perform the log on
	 *
	 * @return the base 2 log of the value
	 */
	[[nodiscard]] static inline double Log2(double Value)
	{
		// Cached value for fast conversions
		constexpr double LogToLog2 = 1.4426950408889634; // 1.0 / Loge(2.0);
		// Do the platform specific log and convert using the cached value
		return Loge(Value) * LogToLog2;
	}

	/**
	* Computes the sine and cosine of a scalar value.
	*
	* @param ScalarSin	Pointer to where the Sin result should be stored
	* @param ScalarCos	Pointer to where the Cos result should be stored
	* @param Value  input angles 
	*/
	template <UE::CFloatingPoint T>
	static constexpr inline void SinCos(std::decay_t<T>* ScalarSin, std::decay_t<T>* ScalarCos, T  Value )
	{
		// Map Value to y in [-pi,pi], x = 2*pi*quotient + remainder.
		T quotient = (UE_INV_PI*0.5f)*Value;
		if (Value >= 0.0f)
		{
			quotient = (T)((int64)(quotient + 0.5f));
		}
		else
		{
			quotient = (T)((int64)(quotient - 0.5f));
		}
		T y = Value - UE_TWO_PI * quotient;

		// Map y to [-pi/2,pi/2] with sin(y) = sin(Value).
		T sign;
		if (y > UE_HALF_PI)
		{
			y = UE_PI - y;
			sign = -1.0f;
		}
		else if (y < -UE_HALF_PI)
		{
			y = -UE_PI - y;
			sign = -1.0f;
		}
		else
		{
			sign = +1.0f;
		}

		T y2 = y * y;

		// 11-degree minimax approximation
		*ScalarSin = ( ( ( ( (-2.3889859e-08f * y2 + 2.7525562e-06f) * y2 - 0.00019840874f ) * y2 + 0.0083333310f ) * y2 - 0.16666667f ) * y2 + 1.0f ) * y;

		// 10-degree minimax approximation
		T p = ( ( ( ( -2.6051615e-07f * y2 + 2.4760495e-05f ) * y2 - 0.0013888378f ) * y2 + 0.041666638f ) * y2 - 0.5f ) * y2 + 1.0f;
		*ScalarCos = sign*p;
	}

	static inline void SinCos(double* ScalarSin, double* ScalarCos, double Value)
	{
		// No approximations for doubles
		*ScalarSin = FMath::Sin(Value);
		*ScalarCos = FMath::Cos(Value);
	}

	template <
		typename T,
		typename U
		UE_REQUIRES(!std::is_same_v<T, U>)
	>
	static UE_FORCEINLINE_HINT void SinCos(T* ScalarSin, T* ScalarCos, U Value)
	{
		SinCos(ScalarSin, ScalarCos, T(Value));
	}


	// Note:  We use FASTASIN_HALF_PI instead of HALF_PI inside of FastASin(), since it was the value that accompanied the minimax coefficients below.
	// It is important to use exactly the same value in all places inside this function to ensure that FastASin(0.0f) == 0.0f.
	// For comparison:
	//		HALF_PI				== 1.57079632679f == 0x3fC90FDB
	//		FASTASIN_HALF_PI	== 1.5707963050f  == 0x3fC90FDA
#define FASTASIN_HALF_PI (1.5707963050f)
	/**
	* Computes the ASin of a scalar value.
	*
	* @param Value  input angle
	* @return ASin of Value
	*/
	[[nodiscard]] static inline float FastAsin(float Value)
	{
		// Clamp input to [-1,1].
		const bool nonnegative = (Value >= 0.0f);
		const float x = FMath::Abs(Value);
		float omx = 1.0f - x;
		if (omx < 0.0f)
		{
			omx = 0.0f;
		}
		const float root = FMath::Sqrt(omx);
		// 7-degree minimax approximation
		float result = ((((((-0.0012624911f * x + 0.0066700901f) * x - 0.0170881256f) * x + 0.0308918810f) * x - 0.0501743046f) * x + 0.0889789874f) * x - 0.2145988016f) * x + FASTASIN_HALF_PI;
		result *= root;  // acos(|x|)
		// acos(x) = pi - acos(-x) when x < 0, asin(x) = pi/2 - acos(x)
		return (nonnegative ? FASTASIN_HALF_PI - result : result - FASTASIN_HALF_PI);
	}
#undef FASTASIN_HALF_PI

	[[nodiscard]] static UE_FORCEINLINE_HINT double FastAsin(double Value)
	{
		// TODO: add fast approximation
		return FMath::Asin(Value);
	}

	// Conversion Functions

	/** 
	 * Converts radians to degrees.
	 * @param	RadVal			Value in radians.
	 * @return					Value in degrees.
	 */
	template<class T>
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT auto RadiansToDegrees(T const& RadVal) -> decltype(RadVal * (180.f / UE_PI))
	{
		return RadVal * (180.f / UE_PI);
	}

	static constexpr UE_FORCEINLINE_HINT float RadiansToDegrees(float const& RadVal) { return RadVal * (180.f / UE_PI); }
	static constexpr UE_FORCEINLINE_HINT double RadiansToDegrees(double const& RadVal) { return RadVal * (180.0 / UE_DOUBLE_PI); }

	/** 
	 * Converts degrees to radians.
	 * @param	DegVal			Value in degrees.
	 * @return					Value in radians.
	 */
	template<class T>
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT auto DegreesToRadians(T const& DegVal) -> decltype(DegVal * (UE_PI / 180.f))
	{
		return DegVal * (UE_PI / 180.f);
	}

	static constexpr UE_FORCEINLINE_HINT float DegreesToRadians(float const& DegVal) { return DegVal * (UE_PI / 180.f); }
	static constexpr UE_FORCEINLINE_HINT double DegreesToRadians(double const& DegVal) { return DegVal * (UE_DOUBLE_PI / 180.0); }

	/** 
	 * Clamps an arbitrary angle to be between the given angles.  Will clamp to nearest boundary.
	 * 
	 * @param MinAngleDegrees	"from" angle that defines the beginning of the range of valid angles (sweeping clockwise)
	 * @param MaxAngleDegrees	"to" angle that defines the end of the range of valid angles
	 * @return Returns clamped angle in the range -180..180.
	 */
	template<typename T>
	[[nodiscard]] static T ClampAngle(T AngleDegrees, T MinAngleDegrees, T MaxAngleDegrees);

	RESOLVE_FLOAT_AMBIGUITY_3_ARGS(ClampAngle);

	/** Find the smallest angle between two headings (in degrees) */
	template <
		typename T,
		typename T2
		UE_REQUIRES(std::is_floating_point_v<T> || std::is_floating_point_v<T2>)
	>
	[[nodiscard]] static constexpr auto FindDeltaAngleDegrees(T A1, T2 A2) -> decltype(A2 - A1)
	{
		// Find the difference
		auto Delta = A2 - A1;

		using DeltaType = decltype(Delta);

		// Return delta in [-180,180] range
		return Wrap(Delta, (DeltaType)-180, (DeltaType)180);
	}

	/** Find the smallest angle between two headings (in radians) */
	template <
		typename T,
		typename T2
		UE_REQUIRES(std::is_floating_point_v<T> || std::is_floating_point_v<T2>)
	>
	[[nodiscard]] static constexpr auto FindDeltaAngleRadians(T A1, T2 A2) -> decltype(A2 - A1)
	{
		// Find the difference
		auto Delta = A2 - A1;

		using DeltaType = decltype(Delta);

		// Return delta in [-PI,PI] range
		return Wrap(Delta, (DeltaType)-UE_PI, (DeltaType)UE_PI);
	}

	/** Given a heading which may be outside the +/- PI range, 'unwind' it back into that range. */
	template <UE::CFloatingPoint T>
	[[nodiscard]] static constexpr T UnwindRadians(T A)
	{
		while(A > UE_PI)
		{
			A -= UE_TWO_PI;
		}

		while(A < -UE_PI)
		{
			A += UE_TWO_PI;
		}

		return A;
	}

	/** Utility to ensure angle is between +/- 180 degrees by unwinding. */
	template <UE::CFloatingPoint T>
	[[nodiscard]] static constexpr T UnwindDegrees(T A)
	{
		while(A > 180.f)
		{
			A -= 360.f;
		}

		while(A < -180.f)
		{
			A += 360.f;
		}

		return A;
	}

	/** 
	 * Given two angles in degrees, 'wind' the rotation in Angle1 so that it avoids >180 degree flips.
	 * Good for winding rotations previously expressed as quaternions into a euler-angle representation.
	 * @param	InAngle0	The first angle that we wind relative to.
	 * @param	InOutAngle1	The second angle that we may wind relative to the first.
	 */
	static CORE_API void WindRelativeAnglesDegrees(float InAngle0, float& InOutAngle1);
	static CORE_API void WindRelativeAnglesDegrees(double InAngle0, double& InOutAngle1);

	/** Returns a new rotation component value
	 *
	 * @param InCurrent is the current rotation value
	 * @param InDesired is the desired rotation value
	 * @param InDeltaRate is the rotation amount to apply
	 *
	 * @return a new rotation component value
	 */
	[[nodiscard]] static CORE_API float FixedTurn(float InCurrent, float InDesired, float InDeltaRate);

	/** Converts given Cartesian coordinate pair to Polar coordinate system. */
	template<typename T>
	static inline void CartesianToPolar(const T X, const T Y, T& OutRad, T& OutAng)
	{
		OutRad = Sqrt(Square(X) + Square(Y));
		OutAng = Atan2(Y, X);
	}
	/** Converts given Cartesian coordinate pair to Polar coordinate system. */
	template<typename T>
	static UE_FORCEINLINE_HINT void CartesianToPolar(const UE::Math::TVector2<T> InCart, UE::Math::TVector2<T>& OutPolar)
	{
		CartesianToPolar(InCart.X, InCart.Y, OutPolar.X, OutPolar.Y);
	}

	/** Converts given Polar coordinate pair to Cartesian coordinate system. */
	template<typename T>
	static inline void PolarToCartesian(const T Rad, const T Ang, T& OutX, T& OutY)
	{
		OutX = Rad * Cos(Ang);
		OutY = Rad * Sin(Ang);
	}
	/** Converts given Polar coordinate pair to Cartesian coordinate system. */
	template<typename T>
	static UE_FORCEINLINE_HINT void PolarToCartesian(const UE::Math::TVector2<T> InPolar, UE::Math::TVector2<T>& OutCart)
	{
		PolarToCartesian(InPolar.X, InPolar.Y, OutCart.X, OutCart.Y);
	}

	/**
	 * Calculates the dotted distance of vector 'Direction' to coordinate system O(AxisX,AxisY,AxisZ).
	 *
	 * Orientation: (consider 'O' the first person view of the player, and 'Direction' a vector pointing to an enemy)
	 * - positive azimuth means enemy is on the right of crosshair. (negative means left).
	 * - positive elevation means enemy is on top of crosshair, negative means below.
	 *
	 * @Note: 'Azimuth' (.X) sign is changed to represent left/right and not front/behind. front/behind is the function's return value.
	 *
	 * @param	OutDotDist	.X = 'Direction' dot AxisX relative to plane (AxisX,AxisZ). (== Cos(Azimuth))
	 *						.Y = 'Direction' dot AxisX relative to plane (AxisX,AxisY). (== Sin(Elevation))
	 * @param	Direction	direction of target.
	 * @param	AxisX		X component of reference system.
	 * @param	AxisY		Y component of reference system.
	 * @param	AxisZ		Z component of reference system.
	 *
	 * @return	true if 'Direction' is facing AxisX (Direction dot AxisX >= 0.f)
	 */
	static CORE_API bool GetDotDistance(FVector2D &OutDotDist, const FVector &Direction, const FVector &AxisX, const FVector &AxisY, const FVector &AxisZ);

	/**
	 * Returns Azimuth and Elevation of vector 'Direction' in coordinate system O(AxisX,AxisY,AxisZ).
	 *
	 * Orientation: (consider 'O' the first person view of the player, and 'Direction' a vector pointing to an enemy)
	 * - positive azimuth means enemy is on the right of crosshair. (negative means left).
	 * - positive elevation means enemy is on top of crosshair, negative means below.
	 *
	 * @param	Direction		Direction of target.
	 * @param	AxisX			X component of reference system.
	 * @param	AxisY			Y component of reference system.
	 * @param	AxisZ			Z component of reference system.
	 *
	 * @return	FVector2D	X = Azimuth angle (in radians) (-PI, +PI)
	 *						Y = Elevation angle (in radians) (-PI/2, +PI/2)
	 */
	[[nodiscard]] static CORE_API FVector2D GetAzimuthAndElevation(const FVector &Direction, const FVector &AxisX, const FVector &AxisY, const FVector &AxisZ);

	// Interpolation Functions

	/** Calculates the percentage along a line from MinValue to MaxValue that Value is. */
	template <UE::CFloatingPoint T, typename T2>
	[[nodiscard]] static constexpr inline auto GetRangePct(T MinValue, T MaxValue, T2 Value)
	{
		// Avoid Divide by Zero.
		// But also if our range is a point, output whether Value is before or after.
		const T Divisor = MaxValue - MinValue;
		if (FMath::IsNearlyZero(Divisor))
		{
			using RetType = decltype(T() / T2());
			return (Value >= MaxValue) ? (RetType)1 : (RetType)0;
		}

		return (Value - MinValue) / Divisor;
	}

	/** Same as above, but taking a 2d vector as the range. */
	template <UE::CFloatingPoint T, typename T2>
	[[nodiscard]] static auto GetRangePct(UE::Math::TVector2<T> const& Range, T2 Value)
	{
		return GetRangePct(Range.X, Range.Y, Value);
	}
	
	/** Basically a Vector2d version of Lerp. */
	template <UE::CFloatingPoint T, typename T2>
	[[nodiscard]] static auto GetRangeValue(UE::Math::TVector2<T> const& Range, T2 Pct)
	{
		return Lerp(Range.X, Range.Y, Pct);
	}

	/** For the given Value clamped to the [Input:Range] inclusive, returns the corresponding percentage in [Output:Range] Inclusive. */
	template<typename T, typename T2>
	[[nodiscard]] static inline auto GetMappedRangeValueClamped(const UE::Math::TVector2<T>& InputRange, const UE::Math::TVector2<T>& OutputRange, const T2 Value)
	{
		using RangePctType = decltype(T() * T2());
		const RangePctType ClampedPct = Clamp<RangePctType>(GetRangePct(InputRange, Value), 0.f, 1.f);
		return GetRangeValue(OutputRange, ClampedPct);
	}

	/** Transform the given Value relative to the input range to the Output Range. */
	template<typename T, typename T2>
	[[nodiscard]] static UE_FORCEINLINE_HINT auto GetMappedRangeValueUnclamped(const UE::Math::TVector2<T>& InputRange, const UE::Math::TVector2<T>& OutputRange, const T2 Value)
	{
		return GetRangeValue(OutputRange, GetRangePct(InputRange, Value));
	}

	template<class T>
	[[nodiscard]] static UE_FORCEINLINE_HINT double GetRangePct(TRange<T> const& Range, T Value)
	{
		return GetRangePct(Range.GetLowerBoundValue(), Range.GetUpperBoundValue(), Value);
	}

	template<class T>
	[[nodiscard]] static UE_FORCEINLINE_HINT T GetRangeValue(TRange<T> const& Range, T Pct)
	{
		return FMath::Lerp<T>(Range.GetLowerBoundValue(), Range.GetUpperBoundValue(), Pct);
	}

	template<class T>
	[[nodiscard]] static inline T GetMappedRangeValueClamped(const TRange<T>& InputRange, const TRange<T>& OutputRange, const T Value)
	{
		const T ClampedPct = FMath::Clamp<T>(GetRangePct(InputRange, Value), 0, 1);
		return GetRangeValue(OutputRange, ClampedPct);
	}

	/** Performs a linear interpolation between two values, Alpha ranges from 0-1 */
	template <
		typename T,
		typename U
		UE_REQUIRES(!TCustomLerp<T>::Value && (std::is_floating_point_v<U> || std::is_same_v<T, U>) && !std::is_same_v<T, bool>)
	>
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT T Lerp( const T& A, const T& B, const U& Alpha )
	{
		return (T)(A + Alpha * (B-A));
	}

	/** Custom lerps defined for those classes not suited to the default implemenation. */
	template <
		typename T,
		typename U
		UE_REQUIRES(TCustomLerp<T>::Value)
	>
	[[nodiscard]] static UE_FORCEINLINE_HINT T Lerp(const T& A, const T& B, const U& Alpha)
	{
		return TCustomLerp<T>::Lerp(A, B, Alpha);
	}

	// Allow passing of differing A/B types.
	template <
		typename T1,
		typename T2,
		typename T3
		UE_REQUIRES(!std::is_same_v<T1, T2> && !TCustomLerp<T1>::Value && !TCustomLerp<T2>::Value)
	>
	[[nodiscard]] static auto Lerp( const T1& A, const T2& B, const T3& Alpha ) -> decltype(A * B)
	{
		using ABType = decltype(A * B);
		return Lerp(ABType(A), ABType(B), Alpha);
	}

	/** Performs a linear interpolation between two values, Alpha ranges from 0-1. Handles full numeric range of T */
	template< class T > 
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT T LerpStable( const T& A, const T& B, double Alpha )
	{
		return (T)((A * (1.0 - Alpha)) + (B * Alpha));
	}

	/** Performs a linear interpolation between two values, Alpha ranges from 0-1. Handles full numeric range of T */
	template< class T >
	[[nodiscard]] static constexpr UE_FORCEINLINE_HINT T LerpStable(const T& A, const T& B, float Alpha)
	{
		return (T)((A * (1.0f - Alpha)) + (B * Alpha));
	}

	// Allow passing of differing A/B types.
	template <
		typename T1,
		typename T2,
		typename T3
		UE_REQUIRES(!std::is_same_v<T1, T2>)
	>
	[[nodiscard]] static auto LerpStable( const T1& A, const T2& B, const T3& Alpha ) -> decltype(A * B)
	{
		using ABType = decltype(A * B);
		return LerpStable(ABType(A), ABType(B), Alpha);
	}
	
	/** Performs a 2D linear interpolation between four values values, FracX, FracY ranges from 0-1 */
	template <
		typename T,
		typename U
		UE_REQUIRES(!TCustomLerp<T>::Value && (std::is_floating_point_v<U> || std::is_same_v<T, U>))
	>
	[[nodiscard]] static constexpr inline T BiLerp(const T& P00,const T& P10,const T& P01,const T& P11, const U& FracX, const U& FracY)
	{
		return Lerp(
			Lerp(P00,P10,FracX),
			Lerp(P01,P11,FracX),
			FracY
			);
	}

	/** Custom lerps defined for those classes not suited to the default implemenation. */
	template <
		typename T,
		typename U
		UE_REQUIRES(TCustomLerp<T>::Value)
	>
	[[nodiscard]] static UE_FORCEINLINE_HINT T BiLerp(const T& P00, const T& P10, const T& P01, const T& P11, const U& FracX, const U& FracY)
	{
		return TCustomLerp<T>::BiLerp(P00, P10, P01, P11, FracX, FracY);
	}

	/**
	 * Performs a cubic interpolation
	 *
	 * @param  P - end points
	 * @param  T - tangent directions at end points
	 * @param  A - distance along spline
	 *
	 * @return  Interpolated value
	 */
	template <
		typename T,
		typename U
		UE_REQUIRES(!TCustomLerp<T>::Value && (std::is_floating_point_v<U> || std::is_same_v<T, U>))
	>
	[[nodiscard]] static constexpr inline T CubicInterp( const T& P0, const T& T0, const T& P1, const T& T1, const U& A )
	{
		const U A2 = A * A;
		const U A3 = A2 * A;

		return T((((2*A3)-(3*A2)+1) * P0) + ((A3-(2*A2)+A) * T0) + ((A3-A2) * T1) + (((-2*A3)+(3*A2)) * P1));
	}

	/** Custom lerps defined for those classes not suited to the default implemenation. */
	template <
		typename T,
		typename U
		UE_REQUIRES(TCustomLerp<T>::Value)
	>
	[[nodiscard]] static UE_FORCEINLINE_HINT T CubicInterp(const T& P0, const T& T0, const T& P1, const T& T1, const U& A)
	{
		return TCustomLerp<T>::CubicInterp(P0, T0, P1, T1, A);
	}

	/**
	 * Performs a first derivative cubic interpolation
	 *
	 * @param  P - end points
	 * @param  T - tangent directions at end points
	 * @param  A - distance along spline
	 *
	 * @return  Interpolated value
	 */
	template <typename T, UE::CFloatingPoint U>
	[[nodiscard]] static constexpr inline T CubicInterpDerivative( const T& P0, const T& T0, const T& P1, const T& T1, const U& A )
	{
		T a = 6.f*P0 + 3.f*T0 + 3.f*T1 - 6.f*P1;
		T b = -6.f*P0 - 4.f*T0 - 2.f*T1 + 6.f*P1;
		T c = T0;

		const U A2 = A * A;

		return T((a * A2) + (b * A) + c);
	}

	/**
	 * Performs a second derivative cubic interpolation
	 *
	 * @param  P - end points
	 * @param  T - tangent directions at end points
	 * @param  A - distance along spline
	 *
	 * @return  Interpolated value
	 */
	template <typename T, UE::CFloatingPoint U>
	[[nodiscard]] static constexpr inline T CubicInterpSecondDerivative( const T& P0, const T& T0, const T& P1, const T& T1, const U& A )
	{
		T a = 12.f*P0 + 6.f*T0 + 6.f*T1 - 12.f*P1;
		T b = -6.f*P0 - 4.f*T0 - 2.f*T1 + 6.f*P1;

		return (a * A) + b;
	}

	/** Interpolate between A and B, applying an ease in function.  Exp controls the degree of the curve. */
	template< class T >
	[[nodiscard]] static inline T InterpEaseIn(const T& A, const T& B, float Alpha, float Exp)
	{
		float const ModifiedAlpha = Pow(Alpha, Exp);
		return Lerp<T>(A, B, ModifiedAlpha);
	}

	/** Interpolate between A and B, applying an ease out function.  Exp controls the degree of the curve. */
	template< class T >
	[[nodiscard]] static inline T InterpEaseOut(const T& A, const T& B, float Alpha, float Exp)
	{
		float const ModifiedAlpha = 1.f - Pow(1.f - Alpha, Exp);
		return Lerp<T>(A, B, ModifiedAlpha);
	}

	/** Interpolate between A and B, applying an ease in/out function.  Exp controls the degree of the curve. */
	template< class T > 
	[[nodiscard]] static inline T InterpEaseInOut( const T& A, const T& B, float Alpha, float Exp )
	{
		return Lerp<T>(A, B, (Alpha < 0.5f) ?
			InterpEaseIn(0.f, 1.f, Alpha * 2.f, Exp) * 0.5f :
			InterpEaseOut(0.f, 1.f, Alpha * 2.f - 1.f, Exp) * 0.5f + 0.5f);
	}

	/** Interpolation between A and B, applying a step function. */
	template< class T >
	[[nodiscard]] static constexpr inline T InterpStep(const T& A, const T& B, float Alpha, int32 Steps)
	{
		if (Steps <= 1 || Alpha <= 0)
		{
			return A;
		}
		else if (Alpha >= 1)
		{
			return B;
		}

		const float StepsAsFloat = static_cast<float>(Steps);
		const float NumIntervals = StepsAsFloat - 1.f;
		float const ModifiedAlpha = FloorToFloat(Alpha * StepsAsFloat) / NumIntervals;
		return Lerp<T>(A, B, ModifiedAlpha);
	}

	/** Interpolation between A and B, applying a sinusoidal in function. */
	template< class T >
	[[nodiscard]] static inline T InterpSinIn(const T& A, const T& B, float Alpha)
	{
		float const ModifiedAlpha = -1.f * Cos(Alpha * UE_HALF_PI) + 1.f;
		return Lerp<T>(A, B, ModifiedAlpha);
	}
	
	/** Interpolation between A and B, applying a sinusoidal out function. */
	template< class T >
	[[nodiscard]] static inline T InterpSinOut(const T& A, const T& B, float Alpha)
	{
		float const ModifiedAlpha = Sin(Alpha * UE_HALF_PI);
		return Lerp<T>(A, B, ModifiedAlpha);
	}

	/** Interpolation between A and B, applying a sinusoidal in/out function. */
	template< class T >
	[[nodiscard]] static inline T InterpSinInOut(const T& A, const T& B, float Alpha)
	{
		return Lerp<T>(A, B, (Alpha < 0.5f) ?
			InterpSinIn(0.f, 1.f, Alpha * 2.f) * 0.5f :
			InterpSinOut(0.f, 1.f, Alpha * 2.f - 1.f) * 0.5f + 0.5f);
	}

	/** Interpolation between A and B, applying an exponential in function. */
	template< class T >
	[[nodiscard]] static inline T InterpExpoIn(const T& A, const T& B, float Alpha)
	{
		float const ModifiedAlpha = (Alpha == 0.f) ? 0.f : Pow(2.f, 10.f * (Alpha - 1.f));
		return Lerp<T>(A, B, ModifiedAlpha);
	}

	/** Interpolation between A and B, applying an exponential out function. */
	template< class T >
	[[nodiscard]] static inline T InterpExpoOut(const T& A, const T& B, float Alpha)
	{
		float const ModifiedAlpha = (Alpha == 1.f) ? 1.f : -Pow(2.f, -10.f * Alpha) + 1.f;
		return Lerp<T>(A, B, ModifiedAlpha);
	}

	/** Interpolation between A and B, applying an exponential in/out function. */
	template< class T >
	[[nodiscard]] static inline T InterpExpoInOut(const T& A, const T& B, float Alpha)
	{
		return Lerp<T>(A, B, (Alpha < 0.5f) ?
			InterpExpoIn(0.f, 1.f, Alpha * 2.f) * 0.5f :
			InterpExpoOut(0.f, 1.f, Alpha * 2.f - 1.f) * 0.5f + 0.5f);
	}

	/** Interpolation between A and B, applying a circular in function. */
	template< class T >
	[[nodiscard]] static inline T InterpCircularIn(const T& A, const T& B, float Alpha)
	{
		float const ModifiedAlpha = -1.f * (Sqrt(1.f - Alpha * Alpha) - 1.f);
		return Lerp<T>(A, B, ModifiedAlpha);
	}

	/** Interpolation between A and B, applying a circular out function. */
	template< class T >
	[[nodiscard]] static inline T InterpCircularOut(const T& A, const T& B, float Alpha)
	{
		Alpha -= 1.f;
		float const ModifiedAlpha = Sqrt(1.f - Alpha  * Alpha);
		return Lerp<T>(A, B, ModifiedAlpha);
	}

	/** Interpolation between A and B, applying a circular in/out function. */
	template< class T >
	[[nodiscard]] static inline T InterpCircularInOut(const T& A, const T& B, float Alpha)
	{
		return Lerp<T>(A, B, (Alpha < 0.5f) ?
			InterpCircularIn(0.f, 1.f, Alpha * 2.f) * 0.5f :
			InterpCircularOut(0.f, 1.f, Alpha * 2.f - 1.f) * 0.5f + 0.5f);
	}

	// Rotator specific interpolation
	// Similar to Lerp, but does not take the shortest path. Allows interpolation over more than 180 degrees.
	template< typename T, typename U >
	[[nodiscard]] static UE::Math::TRotator<T> LerpRange(const UE::Math::TRotator<T>& A, const UE::Math::TRotator<T>& B, U Alpha);

	/*
	 *	Cubic Catmull-Rom Spline interpolation. Based on http://www.cemyuksel.com/research/catmullrom_param/catmullrom.pdf 
	 *	Curves are guaranteed to pass through the control points and are easily chained together.
	 *	Equation supports abitrary parameterization. eg. Uniform=0,1,2,3 ; chordal= |Pn - Pn-1| ; centripetal = |Pn - Pn-1|^0.5
	 *	P0 - The control point preceding the interpolation range.
	 *	P1 - The control point starting the interpolation range.
	 *	P2 - The control point ending the interpolation range.
	 *	P3 - The control point following the interpolation range.
	 *	T0-3 - The interpolation parameters for the corresponding control points.		
	 *	T - The interpolation factor in the range 0 to 1. 0 returns P1. 1 returns P2.
	 */
	template< class U > 
	[[nodiscard]] static constexpr inline U CubicCRSplineInterp(const U& P0, const U& P1, const U& P2, const U& P3, const float T0, const float T1, const float T2, const float T3, const float T)
	{
		//Based on http://www.cemyuksel.com/research/catmullrom_param/catmullrom.pdf 
		float InvT1MinusT0 = 1.0f / (T1 - T0);
		U L01 = ( P0 * ((T1 - T) * InvT1MinusT0) ) + ( P1 * ((T - T0) * InvT1MinusT0) );
		float InvT2MinusT1 = 1.0f / (T2 - T1);
		U L12 = ( P1 * ((T2 - T) * InvT2MinusT1) ) + ( P2 * ((T - T1) * InvT2MinusT1) );
		float InvT3MinusT2 = 1.0f / (T3 - T2);
		U L23 = ( P2 * ((T3 - T) * InvT3MinusT2) ) + ( P3 * ((T - T2) * InvT3MinusT2) );

		float InvT2MinusT0 = 1.0f / (T2 - T0);
		U L012 = ( L01 * ((T2 - T) * InvT2MinusT0) ) + ( L12 * ((T - T0) * InvT2MinusT0) );
		float InvT3MinusT1 = 1.0f / (T3 - T1);
		U L123 = ( L12 * ((T3 - T) * InvT3MinusT1) ) + ( L23 * ((T - T1) * InvT3MinusT1) );

		return  ( ( L012 * ((T2 - T) * InvT2MinusT1) ) + ( L123 * ((T - T1) * InvT2MinusT1) ) );
	}

	/* Same as CubicCRSplineInterp but with additional saftey checks. If the checks fail P1 is returned. **/
	template< class U >
	[[nodiscard]] static constexpr inline U CubicCRSplineInterpSafe(const U& P0, const U& P1, const U& P2, const U& P3, const float T0, const float T1, const float T2, const float T3, const float T)
	{
		//Based on http://www.cemyuksel.com/research/catmullrom_param/catmullrom.pdf 
		float T1MinusT0 = (T1 - T0);
		float T2MinusT1 = (T2 - T1);
		float T3MinusT2 = (T3 - T2);
		float T2MinusT0 = (T2 - T0);
		float T3MinusT1 = (T3 - T1);
		if (FMath::IsNearlyZero(T1MinusT0) || FMath::IsNearlyZero(T2MinusT1) || FMath::IsNearlyZero(T3MinusT2) || FMath::IsNearlyZero(T2MinusT0) || FMath::IsNearlyZero(T3MinusT1))
		{
			//There's going to be a divide by zero here so just bail out and return P1
			return P1;
		}

		float InvT1MinusT0 = 1.0f / T1MinusT0;
		U L01 = (P0 * ((T1 - T) * InvT1MinusT0)) + (P1 * ((T - T0) * InvT1MinusT0));
		float InvT2MinusT1 = 1.0f / T2MinusT1;
		U L12 = (P1 * ((T2 - T) * InvT2MinusT1)) + (P2 * ((T - T1) * InvT2MinusT1));
		float InvT3MinusT2 = 1.0f / T3MinusT2;
		U L23 = (P2 * ((T3 - T) * InvT3MinusT2)) + (P3 * ((T - T2) * InvT3MinusT2));

		float InvT2MinusT0 = 1.0f / T2MinusT0;
		U L012 = (L01 * ((T2 - T) * InvT2MinusT0)) + (L12 * ((T - T0) * InvT2MinusT0));
		float InvT3MinusT1 = 1.0f / T3MinusT1;
		U L123 = (L12 * ((T3 - T) * InvT3MinusT1)) + (L23 * ((T - T1) * InvT3MinusT1));

		return  ((L012 * ((T2 - T) * InvT2MinusT1)) + (L123 * ((T - T1) * InvT2MinusT1)));
	}


	// Special-case interpolation

	/** Interpolate a normal vector Current to Target, by interpolating the angle between those vectors with constant step. */
	[[nodiscard]] static CORE_API FVector VInterpNormalRotationTo(const FVector& Current, const FVector& Target, float DeltaTime, float RotationSpeedDegrees);

	/** Interpolate vector from Current to Target with constant step */
	[[nodiscard]] static CORE_API FVector VInterpConstantTo(const FVector& Current, const FVector& Target, float DeltaTime, float InterpSpeed);

	/** Interpolate vector from Current to Target. Scaled by distance to Target, so it has a strong start speed and ease out. */
	[[nodiscard]] static CORE_API FVector VInterpTo( const FVector& Current, const FVector& Target, float DeltaTime, float InterpSpeed );
	
	/** Interpolate vector2D from Current to Target with constant step */
	[[nodiscard]] static CORE_API FVector2D Vector2DInterpConstantTo( const FVector2D& Current, const FVector2D& Target, float DeltaTime, float InterpSpeed );

	/** Interpolate vector2D from Current to Target. Scaled by distance to Target, so it has a strong start speed and ease out. */
	template <typename T>
	[[nodiscard]] static CORE_API UE::Math::TVector2<T> Vector2DInterpTo( const UE::Math::TVector2<T>& Current, const UE::Math::TVector2<T>& Target, float DeltaTime, float InterpSpeed );

	/** Interpolate rotator from Current to Target with constant step */
	[[nodiscard]] static CORE_API FRotator RInterpConstantTo( const FRotator& Current, const FRotator& Target, float DeltaTime, float InterpSpeed);

	/** Interpolate rotator from Current to Target. Scaled by distance to Target, so it has a strong start speed and ease out. */
	[[nodiscard]] static CORE_API FRotator RInterpTo( const FRotator& Current, const FRotator& Target, float DeltaTime, float InterpSpeed);

	/** Interpolate float from Current to Target with constant step */
	template<typename T1, typename T2 = T1, typename T3 = T2, typename T4 = T3>
	[[nodiscard]] static auto FInterpConstantTo( T1 Current, T2 Target, T3 DeltaTime, T4 InterpSpeed )
	{
		static_assert(!std::is_same_v<T1, bool> && !std::is_same_v<T2, bool>, "Boolean types may not be interpolated");
		using RetType = decltype(T1() * T2() * T3() * T4());
	
		const RetType Dist = Target - Current;

		// If distance is too small, just set the desired location
		if( FMath::Square(Dist) < UE_SMALL_NUMBER )
		{
			return static_cast<RetType>(Target);
		}

		const RetType Step = InterpSpeed * DeltaTime;
		return Current + FMath::Clamp(Dist, -Step, Step);
	}

	/** Interpolate float from Current to Target. Scaled by distance to Target, so it has a strong start speed and ease out. */
	template<typename T1, typename T2 = T1, typename T3 = T2, typename T4 = T3>
	[[nodiscard]] static auto FInterpTo( T1  Current, T2 Target, T3 DeltaTime, T4 InterpSpeed )
	{
		static_assert(!std::is_same_v<T1, bool> && !std::is_same_v<T2, bool>, "Boolean types may not be interpolated");
		using RetType = decltype(T1() * T2() * T3() * T4());
	
		// If no interp speed, jump to target value
		if( InterpSpeed <= 0.f )
		{
			return static_cast<RetType>(Target);
		}

		// Distance to reach
		const RetType Dist = Target - Current;

		// If distance is too small, just set the desired location
		if( FMath::Square(Dist) < UE_SMALL_NUMBER )
		{
			return static_cast<RetType>(Target);
		}

		// Delta Move, Clamp so we do not over shoot.
		const RetType DeltaMove = Dist * FMath::Clamp<RetType>(DeltaTime * InterpSpeed, 0.f, 1.f);

		return Current + DeltaMove;				
	}

	/** Interpolate Linear Color from Current to Target. Scaled by distance to Target, so it has a strong start speed and ease out. */
	[[nodiscard]] static CORE_API FLinearColor CInterpTo(const FLinearColor& Current, const FLinearColor& Target, float DeltaTime, float InterpSpeed);

	/** Interpolate quaternion from Current to Target with constant step (in radians) */
	template< class T >
	[[nodiscard]] static CORE_API UE::Math::TQuat<T> QInterpConstantTo(const UE::Math::TQuat<T>& Current, const UE::Math::TQuat<T>& Target, float DeltaTime, float InterpSpeed);

	/** Interpolate quaternion from Current to Target. Scaled by angle to Target, so it has a strong start speed and ease out. */
	template< class T >
	[[nodiscard]] static CORE_API UE::Math::TQuat<T> QInterpTo(const UE::Math::TQuat<T>& Current, const UE::Math::TQuat<T>& Target, float DeltaTime, float InterpSpeed);

	/**
	* Returns an approximation of Exp(-X) based on a Taylor expansion that has had the coefficients adjusted (using
	* optimisation) to minimise the error in the range 0 < X < 1, which is below 0.1%. Note that it returns exactly 1
	* when X is 0, and the return value is greater than the real value for values of X > 1 (but it still tends
	* to zero for large X). 
	*/
	template<class T>
	[[nodiscard]] static constexpr T InvExpApprox(T X)
	{
		constexpr T A(1.00746054f); // 1 / 1! in Taylor series
		constexpr T B(0.45053901f); // 1 / 2! in Taylor series
		constexpr T C(0.25724632f); // 1 / 3! in Taylor series
		return 1 / (1 + A * X + B * X * X + C * X * X * X);		
	}
	
	/**
	* Smooths a value using exponential damping towards a target. Works for any type that supports basic arithmetic operations.
	* 
	* An approximation is used that is accurate so long as InDeltaTime < 0.5 * InSmoothingTime
	* 
	* @param  InOutValue      The value to be smoothed
	* @param  InTargetValue   The target to smooth towards
	* @param  InDeltaTime     Time interval
	* @param  InSmoothingTime Timescale over which to smooth. Larger values result in more smoothed behaviour. Can be zero.
	*/
	template< class T >
    static constexpr void ExponentialSmoothingApprox(
        T&          InOutValue,
        const T&    InTargetValue,
        const float InDeltaTime,
        const float InSmoothingTime)
	{
		if (InSmoothingTime > UE_KINDA_SMALL_NUMBER)
		{
			const float A = InDeltaTime / InSmoothingTime;
			float Exp = InvExpApprox(A);
			InOutValue = InTargetValue + (InOutValue - InTargetValue) * Exp;
		}
		else
		{
			InOutValue = InTargetValue;
		}
	}

	/**
	 * Smooths a value using a critically damped spring. Works for any type that supports basic arithmetic operations.
	 * 
	 * Note that InSmoothingTime is the time lag when tracking constant motion (so if you tracked a predicted position 
	 * TargetVel * InSmoothingTime ahead, you'd match the target position)
	 *
	 * An approximation is used that is accurate so long as InDeltaTime < 0.5 * InSmoothingTime
	 * 
	 * When starting from zero velocity, the maximum velocity is reached after time InSmoothingTime * 0.5
	 *
	 * @param  InOutValue        The value to be smoothed
	 * @param  InOutValueRate    The rate of change of the value
	 * @param  InTargetValue     The target to smooth towards
	 * @param  InTargetValueRate The target rate of change smooth towards. Note that if this is discontinuous, then the output will have discontinuous velocity too.
	 * @param  InDeltaTime       Time interval
	 * @param  InSmoothingTime   Timescale over which to smooth. Larger values result in more smoothed behaviour. Can be zero.
	 */
	template< class T >
	static constexpr void CriticallyDampedSmoothing(
		T&          InOutValue,
		T&          InOutValueRate,
		const T&    InTargetValue,
	    const T&    InTargetValueRate,
		const float InDeltaTime,
		const float InSmoothingTime)
	{
		if (InSmoothingTime > UE_KINDA_SMALL_NUMBER)
		{
			// The closed form solution for critically damped motion towards zero is:
			//
			// x = ( x0 + t (v0 + w x0) ) exp(-w t)
			//
			// where w is the natural frequency, x0, v0 are x and dx/dt at time 0 (e.g. Brian Stone - Chatter and Machine Tools pdf)
			//
			// Differentiate to get velocity (remember d(v0)/dt = 0):
			//
			// v = ( v0 - w t (v0 + w x0) ) exp(-w t)
			//
			// We're damping towards a target, so convert - i.e. x = value - target
			//
			// The target velocity turns into an offset to the position
			T AdjustedTarget = InTargetValue + InTargetValueRate * InSmoothingTime;

			const float W = 2.0f / InSmoothingTime;
			const float A = W * InDeltaTime;
			// Approximation requires A < 1, so DeltaTime < SmoothingTime / 2
			const float Exp = InvExpApprox(A);
			T X0 = InOutValue - AdjustedTarget;
			const T B = (InOutValueRate + X0 * W) * InDeltaTime;
			InOutValue = AdjustedTarget + (X0 + B) * Exp;
			InOutValueRate = (InOutValueRate - B * W) * Exp;
		}
		else if (InDeltaTime > 0.0f)
		{
			InOutValueRate = (InTargetValue - InOutValue) / InDeltaTime;
			InOutValue = InTargetValue;
		}
		else
		{
			InOutValue = InTargetValue;
			InOutValueRate = T(0);
		}
	}

	/**
	* Smooths a value using a spring damper towards a target.
	* 
	* The implementation uses approximations for Exp/Sin/Cos. These are accurate for all sensible values of 
	* InUndampedFrequency and DampingRatio so long as InDeltaTime < 1 / InUndampedFrequency (approximately), but
	* are generally well behaved even for larger timesteps etc. 
	* 
	* @param  InOutValue          The value to be smoothed
	* @param  InOutValueRate      The rate of change of the value
	* @param  InTargetValue       The target to smooth towards
	* @param  InTargetValueRate   The target rate of change smooth towards. Note that if this is discontinuous, then the output will have discontinuous velocity too.
	* @param  InDeltaTime         Time interval
	* @param  InUndampedFrequency Oscillation frequency when there is no damping. Proportional to the square root of the spring stiffness.
	* @param  InDampingRatio      1 is critical damping. <1 results in under-damped motion (i.e. with overshoot), and >1 results in over-damped motion. 
	*/
	template< class T >
	static void SpringDamper(
	    T&          InOutValue,
	    T&          InOutValueRate,
	    const T&    InTargetValue,
	    const T&    InTargetValueRate,
	    const float InDeltaTime,
	    const float InUndampedFrequency,
	    const float InDampingRatio)
	{
		if (InDeltaTime <= 0.0f)
		{
			return;
		}

		float W = InUndampedFrequency * UE_TWO_PI;
		// Handle special cases
		if (W < UE_SMALL_NUMBER) // no strength which means no damping either
		{
			InOutValue += InOutValueRate * InDeltaTime;
			return;
		}
		else if (InDampingRatio < UE_SMALL_NUMBER) // No damping at all
		{
			T Err = InOutValue - InTargetValue;
			const T B = InOutValueRate / W;
			float S, C;
			FMath::SinCos(&S, &C, W * InDeltaTime);
			InOutValue = InTargetValue + Err * C + B * S;
			InOutValueRate = InOutValueRate * C - Err * (W * S);
			return;
		}

		// Target velocity turns into an offset to the position
		float SmoothingTime = 2.0f / W;
		T AdjustedTarget = InTargetValue + InTargetValueRate * (InDampingRatio * SmoothingTime);
 		T Err = InOutValue - AdjustedTarget;

		// Handle the cases separately
		if (InDampingRatio > 1.0f) // Overdamped
		{
			const float WD = W * FMath::Sqrt(FMath::Square(InDampingRatio) - 1.0f);
			const T C2 = -(InOutValueRate + (W * InDampingRatio - WD) * Err) / (2.0f * WD);
			const T C1 = Err - C2;
			const float A1 = (WD - InDampingRatio * W);
			const float A2 = -(WD + InDampingRatio * W);
			// Note that A1 and A2 will always be negative. We will use an approximation for 1/Exp(-A * DeltaTime).
			const float A1_DT = -A1 * InDeltaTime;
			const float A2_DT = -A2 * InDeltaTime;
			// This approximation in practice will be good for all DampingRatios
			const float E1 = InvExpApprox(A1_DT);
			// As DampingRatio gets big, this approximation gets worse, but mere inaccuracy for overdamped motion is
			// not likely to be important, since we end up with 1 / BigNumber
			const float E2 = InvExpApprox(A2_DT);
			InOutValue = AdjustedTarget + E1 * C1 + E2 * C2;
			InOutValueRate = E1 * C1 * A1 + E2 * C2 * A2;
		}
		else if (InDampingRatio < 1.0f) // Underdamped
		{
			const float WD = W * FMath::Sqrt(1.0f - FMath::Square(InDampingRatio));
			const T A = Err;
			const T B = (InOutValueRate + Err * (InDampingRatio * W)) / WD;
			float S, C;
			FMath::SinCos(&S, &C, WD * InDeltaTime);
			const float E0 = InDampingRatio * W * InDeltaTime;
			// Needs E0 < 1 so DeltaTime < SmoothingTime / (2 * DampingRatio * Sqrt(1 - DampingRatio^2))
			const float E = InvExpApprox(E0);
			InOutValue = E * (A * C + B * S);
			InOutValueRate = -InOutValue * InDampingRatio * W;
			InOutValueRate += E * (B * (WD * C) - A * (WD * S));
			InOutValue += AdjustedTarget;
		}
		else // Critical damping
		{
			const T& C1 = Err;
			T C2 = InOutValueRate + Err * W;
			const float E0 = W * InDeltaTime;
			// Needs E0 < 1 so InDeltaTime < SmoothingTime / 2 
			float E = InvExpApprox(E0);
			InOutValue = AdjustedTarget + (C1 + C2 * InDeltaTime) * E;
			InOutValueRate = (C2 - C1 * W - C2 * (W * InDeltaTime)) * E;
		}
	}

	/**
	* Smooths a value using a spring damper towards a target.
	* 
	* The implementation uses approximations for Exp/Sin/Cos. These are accurate for all sensible values of 
	* DampingRatio and InSmoothingTime so long as InDeltaTime < 0.5 * InSmoothingTime, but are generally well behaved 
	* even for larger timesteps etc. 
	* 
	* @param  InOutValue        The value to be smoothed
	* @param  InOutValueRate    The rate of change of the value
	* @param  InTargetValue     The target to smooth towards
	* @param  InTargetValueRate The target rate of change smooth towards. Note that if this is discontinuous, then the output will have discontinuous velocity too.
	* @param  InDeltaTime       Time interval
	* @param  InSmoothingTime   Timescale over which to smooth. Larger values result in more smoothed behaviour. Can be zero.
	* @param  InDampingRatio    1 is critical damping. <1 results in under-damped motion (i.e. with overshoot), and >1 results in over-damped motion. 
	*/
	template< class T >
	static void SpringDamperSmoothing(
		T&          InOutValue,
		T&          InOutValueRate,
		const T&    InTargetValue,
		const T&    InTargetValueRate,
		const float InDeltaTime,
		const float InSmoothingTime,
		const float InDampingRatio)
	{
		if (InSmoothingTime < UE_SMALL_NUMBER)
		{
			if (InDeltaTime <= 0.0f)
			{
				return;
			}
			InOutValueRate = (InTargetValue - InOutValue) / InDeltaTime;
			InOutValue = InTargetValue;
			return;
		}

		// Undamped frequency
		float UndampedFrequency = 1.0f / (UE_PI * InSmoothingTime);
		SpringDamper(InOutValue, InOutValueRate, InTargetValue, InTargetValueRate, InDeltaTime, UndampedFrequency, InDampingRatio);
	}

	/**
	 * Simple function to create a pulsating scalar value
	 *
	 * @param  InCurrentTime  Current absolute time
	 * @param  InPulsesPerSecond  How many full pulses per second?
	 * @param  InPhase  Optional phase amount, between 0.0 and 1.0 (to synchronize pulses)
	 *
	 * @return  Pulsating value (0.0-1.0)
	 */
	[[nodiscard]] static float MakePulsatingValue( const double InCurrentTime, const float InPulsesPerSecond, const float InPhase = 0.0f )
	{
		return 0.5f + 0.5f * FMath::Sin( ( ( 0.25f + InPhase ) * UE_TWO_PI ) + ( (float)InCurrentTime * UE_TWO_PI ) * InPulsesPerSecond );
	}

	// Geometry intersection 

	/**
	 * Find the intersection of a ray and a plane.  The ray has a start point with an infinite length.  Assumes that the
	 * line and plane do indeed intersect; you must make sure they're not parallel before calling.
	 *
	 * @param RayOrigin	The start point of the ray
	 * @param RayDirection	The direction the ray is pointing (normalized vector)
	 * @param Plane	The plane to intersect with
	 *
	 * @return The point of intersection between the ray and the plane.
	 */
	template<typename FReal>
	[[nodiscard]] static UE::Math::TVector<FReal> RayPlaneIntersection(const UE::Math::TVector<FReal>& RayOrigin, const UE::Math::TVector<FReal>& RayDirection, const UE::Math::TPlane<FReal>& Plane);

	/**
	 * Find the intersection of a ray and a plane.  The ray has a start point with an infinite length.  Assumes that the
	 * line and plane do indeed intersect; you must make sure they're not parallel before calling.
	 *
	 * @param RayOrigin	The start point of the ray
	 * @param RayDirection	The direction the ray is pointing (normalized vector)
	 * @param Plane	The plane to intersect with
	 *
	 * @return The distance parameter along ray of the point of intersection between the ray and the plane.
	 */
	template<typename FReal>
	[[nodiscard]] static FReal RayPlaneIntersectionParam(const UE::Math::TVector<FReal>& RayOrigin, const UE::Math::TVector<FReal>& RayDirection, const UE::Math::TPlane<FReal>& Plane);

	/**
	 * Find the intersection of a line and an offset plane. Assumes that the
	 * line and plane do indeed intersect; you must make sure they're not
	 * parallel before calling.
	 *
	 * @param Point1 the first point defining the line
	 * @param Point2 the second point defining the line
	 * @param PlaneOrigin the origin of the plane
	 * @param PlaneNormal the normal of the plane
	 *
	 * @return The point of intersection between the line and the plane.
	 */
	template<typename FReal>
	[[nodiscard]] static UE::Math::TVector<FReal> LinePlaneIntersection(const UE::Math::TVector<FReal>& Point1, const UE::Math::TVector<FReal>& Point2, const UE::Math::TVector<FReal>& PlaneOrigin, const UE::Math::TVector<FReal>& PlaneNormal);

	/**
	 * Find the intersection of a line and a plane. Assumes that the line and
	 * plane do indeed intersect; you must make sure they're not parallel before
	 * calling.
	 *
	 * @param Point1 the first point defining the line
	 * @param Point2 the second point defining the line
	 * @param Plane the plane
	 *
	 * @return The point of intersection between the line and the plane.
	 */
	template<typename FReal>
	[[nodiscard]] static UE::Math::TVector<FReal> LinePlaneIntersection(const UE::Math::TVector<FReal>& Point1, const UE::Math::TVector<FReal>& Point2, const UE::Math::TPlane<FReal>& Plane);


	// @parma InOutScissorRect should be set to View.ViewRect before the call
	// @return 0: light is not visible, 1:use scissor rect, 2: no scissor rect needed
	[[nodiscard]] static CORE_API uint32 ComputeProjectedSphereScissorRect(FIntRect& InOutScissorRect, FVector SphereOrigin, float Radius, FVector ViewOrigin, const FMatrix& ViewMatrix, const FMatrix& ProjMatrix);

	// @param ConeOrigin Cone origin
	// @param ConeDirection Cone direction
	// @param ConeRadius Cone Radius
	// @param CosConeAngle Cos of the cone angle
	// @param SinConeAngle Sin of the cone angle
	// @return Minimal bounding sphere encompassing given cone
	template<typename FReal>
	[[nodiscard]] static UE::Math::TSphere<FReal> ComputeBoundingSphereForCone(UE::Math::TVector<FReal> const& ConeOrigin, UE::Math::TVector<FReal> const& ConeDirection, FReal ConeRadius, FReal CosConeAngle, FReal SinConeAngle);

	/** 
	 * Determine if a plane and an AABB intersect
	 * @param P - the plane to test
	 * @param AABB - the axis aligned bounding box to test
	 * @return if collision occurs
	 */
	[[nodiscard]] static CORE_API bool PlaneAABBIntersection(const FPlane& P, const FBox& AABB);

	/**
	 * Determine the position of an AABB relative to a plane:
	 * completely above (in the direction of the normal of the plane), completely below or intersects it
	 * @param P - the plane to test
	 * @param AABB - the axis aligned bounding box to test
	 * @return -1 if below, 1 if above, 0 if intersects
	 */
	[[nodiscard]] static CORE_API int32 PlaneAABBRelativePosition(const FPlane& P, const FBox& AABB);

	/**
	 * Performs a sphere vs box intersection test using Arvo's algorithm:
	 *
	 *	for each i in (x, y, z)
	 *		if (SphereCenter(i) < BoxMin(i)) d2 += (SphereCenter(i) - BoxMin(i)) ^ 2
	 *		else if (SphereCenter(i) > BoxMax(i)) d2 += (SphereCenter(i) - BoxMax(i)) ^ 2
	 *
	 * @param SphereCenter the center of the sphere being tested against the AABB
	 * @param RadiusSquared the size of the sphere being tested
	 * @param AABB the box being tested against
	 *
	 * @return Whether the sphere/box intersect or not.
	 */
	template<typename FReal>
	[[nodiscard]] static bool SphereAABBIntersection(const UE::Math::TVector<FReal>& SphereCenter,const FReal RadiusSquared,const UE::Math::TBox<FReal>& AABB);

	/**
	 * Converts a sphere into a point plus radius squared for the test above
	 */
	template<typename FReal>
	[[nodiscard]] static bool SphereAABBIntersection(const UE::Math::TSphere<FReal>& Sphere, const UE::Math::TBox<FReal>& AABB);

	/** Determines whether a point is inside a box. */
	template<typename FReal>
	[[nodiscard]] static bool PointBoxIntersection( const UE::Math::TVector<FReal>& Point, const UE::Math::TBox<FReal>& Box );

	/** Determines whether a line intersects a box. */
	template<typename FReal>
	[[nodiscard]] static bool LineBoxIntersection( const UE::Math::TBox<FReal>& Box, const UE::Math::TVector<FReal>& Start, const UE::Math::TVector<FReal>& End, const UE::Math::TVector<FReal>& Direction );

	/** Determines whether a line intersects a box. This overload avoids the need to do the reciprocal every time. */
	template<typename FReal>
	[[nodiscard]] static bool LineBoxIntersection( const UE::Math::TBox<FReal>& Box, const UE::Math::TVector<FReal>& Start, const UE::Math::TVector<FReal>& End, const UE::Math::TVector<FReal>& Direction, const UE::Math::TVector<FReal>& OneOverDirection );

	/** Determines whether a line intersects a 2D box. */
	template<typename FReal>
	[[nodiscard]] static bool LineBoxIntersection2D( const UE::Math::TBox2<FReal>& Box, const UE::Math::TVector2<FReal>& Start, const UE::Math::TVector2<FReal>& End );

	/* Swept-Box vs Box test */
	static CORE_API bool LineExtentBoxIntersection(const FBox& inBox, const FVector& Start, const FVector& End, const FVector& Extent, FVector& HitLocation, FVector& HitNormal, float& HitTime);

	/** Determines whether a line intersects a sphere. */
	template<typename FReal>
	[[nodiscard]] static bool LineSphereIntersection(const UE::Math::TVector<FReal>& Start,const UE::Math::TVector<FReal>& Dir, FReal Length,const UE::Math::TVector<FReal>& Origin, FReal Radius);

	/**
	 * Assumes the cone tip is at 0,0,0 (means the SphereCenter is relative to the cone tip)
	 * @return true: cone and sphere do intersect, false otherwise
	 */
	[[nodiscard]] static CORE_API bool SphereConeIntersection(const FVector& SphereCenter, float SphereRadius, const FVector& ConeAxis, float ConeAngleSin, float ConeAngleCos);

	/** Find the point on the line segment from LineStart to LineEnd which is closest to Point */
	template<typename T>
	[[nodiscard]] static CORE_API UE::Math::TVector<T> ClosestPointOnLine(const UE::Math::TVector<T>& LineStart, const UE::Math::TVector<T>& LineEnd, const UE::Math::TVector<T>& Point);

	/** Find the point on the infinite line between two points (LineStart, LineEnd) which is closest to Point */
	[[nodiscard]] static CORE_API FVector ClosestPointOnInfiniteLine(const FVector& LineStart, const FVector& LineEnd, const FVector& Point);

	/** Compute intersection point of three planes. Return 1 if valid, 0 if infinite. */
	template<typename FReal>
	[[nodiscard]] static bool IntersectPlanes3(UE::Math::TVector<FReal>& I, const UE::Math::TPlane<FReal>& P1, const UE::Math::TPlane<FReal>& P2, const UE::Math::TPlane<FReal>& P3 );

	/**
	 * Compute intersection point and direction of line joining two planes.
	 * Return 1 if valid, 0 if infinite.
	 */
	template<typename FReal>
	[[nodiscard]] static bool IntersectPlanes2(UE::Math::TVector<FReal>& I, UE::Math::TVector<FReal>& D, const UE::Math::TPlane<FReal>& P1, const UE::Math::TPlane<FReal>& P2);

	/**
	 * Calculates the distance of a given Point in world space to a given line,
	 * defined by the vector couple (Origin, Direction).
	 *
	 * @param	Point				Point to check distance to line
	 * @param	Direction			Vector indicating the direction of the line. Not required to be normalized.
	 * @param	Origin				Point of reference used to calculate distance
	 * @param	OutClosestPoint	optional point that represents the closest point projected onto Axis
	 *
	 * @return	distance of Point from line defined by (Origin, Direction)
	 */
	static CORE_API float PointDistToLine(const FVector &Point, const FVector &Direction, const FVector &Origin, FVector &OutClosestPoint);
	[[nodiscard]] static CORE_API float PointDistToLine(const FVector &Point, const FVector &Direction, const FVector &Origin);

	/**
	 * Returns closest point on a segment to a given point.
	 * The idea is to project point on line formed by segment.
	 * Then we see if the closest point on the line is outside of segment or inside.
	 *
	 * @param	Point			point for which we find the closest point on the segment
	 * @param	StartPoint		StartPoint of segment
	 * @param	EndPoint		EndPoint of segment
	 *
	 * @return	point on the segment defined by (StartPoint, EndPoint) that is closest to Point.
	 */
	template <typename T>
	[[nodiscard]] static CORE_API UE::Math::TVector<T> ClosestPointOnSegment(const UE::Math::TVector<T>&Point, const UE::Math::TVector<T>&StartPoint, const UE::Math::TVector<T>&EndPoint);

	/**
	* FVector2D version of ClosestPointOnSegment.
	* Returns closest point on a segment to a given 2D point.
	* The idea is to project point on line formed by segment.
	* Then we see if the closest point on the line is outside of segment or inside.
	*
	* @param	Point			point for which we find the closest point on the segment
	* @param	StartPoint		StartPoint of segment
	* @param	EndPoint		EndPoint of segment
	*
	* @return	point on the segment defined by (StartPoint, EndPoint) that is closest to Point.
	*/
	template <typename T>
	[[nodiscard]] static CORE_API UE::Math::TVector2<T> ClosestPointOnSegment2D(const UE::Math::TVector2<T>& Point, const UE::Math::TVector2<T>& StartPoint, const UE::Math::TVector2<T>& EndPoint);

	/**
	 * Returns distance from a point to the closest point on a segment.
	 *
	 * @param	Point			point to check distance for
	 * @param	StartPoint		StartPoint of segment
	 * @param	EndPoint		EndPoint of segment
	 *
	 * @return	closest distance from Point to segment defined by (StartPoint, EndPoint).
	 */
	[[nodiscard]] static CORE_API float PointDistToSegment(const FVector &Point, const FVector &StartPoint, const FVector &EndPoint);

	/**
	 * Returns square of the distance from a point to the closest point on a segment.
	 *
	 * @param	Point			point to check distance for
	 * @param	StartPoint		StartPoint of segment
	 * @param	EndPoint		EndPoint of segment
	 *
	 * @return	square of the closest distance from Point to segment defined by (StartPoint, EndPoint).
	 */
	[[nodiscard]] static CORE_API float PointDistToSegmentSquared(const FVector &Point, const FVector &StartPoint, const FVector &EndPoint);

	/** 
	 * Find closest points between 2 segments.
	 *
	 * If either segment may have a length of 0, use SegmentDistToSegmentSafe instance.
	 *
	 * @param	(A1, B1)	defines the first segment.
	 * @param	(A2, B2)	defines the second segment.
	 * @param	OutP1		Closest point on segment 1 to segment 2.
	 * @param	OutP2		Closest point on segment 2 to segment 1.
	 */
	static CORE_API void SegmentDistToSegment(FVector A1, FVector B1, FVector A2, FVector B2, FVector& OutP1, FVector& OutP2);

	/** 
	 * Find closest points between 2 segments.
	 *
	 * This is the safe version, and will check both segments' lengths.
	 * Use this if either (or both) of the segments lengths may be 0.
	 *
	 * @param	(A1, B1)	defines the first segment.
	 * @param	(A2, B2)	defines the second segment.
	 * @param	OutP1		Closest point on segment 1 to segment 2.
	 * @param	OutP2		Closest point on segment 2 to segment 1.
	 */
	template<typename T>
	static CORE_API void SegmentDistToSegmentSafe(UE::Math::TVector<T> A1, UE::Math::TVector<T> B1, UE::Math::TVector<T> A2, UE::Math::TVector<T> B2, UE::Math::TVector<T>& OutP1, UE::Math::TVector<T>& OutP2);

	/**
	 * returns the time (t) of the intersection of the passed segment and a plane (could be <0 or >1)
	 * @param StartPoint - start point of segment
	 * @param EndPoint   - end point of segment
	 * @param Plane		- plane to intersect with
	 * @return time(T) of intersection
	 */
	[[nodiscard]] static CORE_API float GetTForSegmentPlaneIntersect(const FVector& StartPoint, const FVector& EndPoint, const FPlane& Plane);

	/**
	 * Returns true if there is an intersection between the segment specified by StartPoint and Endpoint, and
	 * the plane on which polygon Plane lies. If there is an intersection, the point is placed in out_IntersectionPoint
	 * @param StartPoint - start point of segment
	 * @param EndPoint   - end point of segment
	 * @param Plane		- plane to intersect with
	 * @param out_IntersectionPoint - out var for the point on the segment that intersects the mesh (if any)
	 * @return true if intersection occurred
	 */
	static CORE_API bool SegmentPlaneIntersection(const FVector& StartPoint, const FVector& EndPoint, const FPlane& Plane, FVector& out_IntersectionPoint);


	/**
	* Returns true if there is an intersection between the segment specified by StartPoint and Endpoint, and
	* the Triangle defined by A, B and C. If there is an intersection, the point is placed in out_IntersectionPoint
	* @param StartPoint - start point of segment
	* @param EndPoint   - end point of segment
	* @param A, B, C	- points defining the triangle 
	* @param OutIntersectPoint - out var for the point on the segment that intersects the triangle (if any)
	* @param OutTriangleNormal - out var for the triangle normal
	* @return true if intersection occurred
	*/
	static CORE_API bool SegmentTriangleIntersection(const FVector& StartPoint, const FVector& EndPoint, const FVector& A, const FVector& B, const FVector& C, FVector& OutIntersectPoint, FVector& OutTriangleNormal);

	/**
	 * Returns true if there is an intersection between the segment specified by SegmentStartA and SegmentEndA, and
	 * the segment specified by SegmentStartB and SegmentEndB, in 2D space. If there is an intersection, the point is placed in out_IntersectionPoint
	 * @param SegmentStartA - start point of first segment
	 * @param SegmentEndA   - end point of first segment
	 * @param SegmentStartB - start point of second segment
	 * @param SegmentEndB   - end point of second segment
	 * @param out_IntersectionPoint - out var for the intersection point (if any)
	 * @return true if intersection occurred
	 */
	static CORE_API bool SegmentIntersection2D(const FVector& SegmentStartA, const FVector& SegmentEndA, const FVector& SegmentStartB, const FVector& SegmentEndB, FVector& out_IntersectionPoint);


	/**
	 * Returns closest point on a triangle to a point.
	 * The idea is to identify the halfplanes that the point is
	 * in relative to each triangle segment "plane"
	 *
	 * @param	Point			point to check distance for
	 * @param	A,B,C			counter clockwise ordering of points defining a triangle
	 *
	 * @return	Point on triangle ABC closest to given point
	 */
	[[nodiscard]] static CORE_API FVector ClosestPointOnTriangleToPoint(const FVector& Point, const FVector& A, const FVector& B, const FVector& C);

	/**
	 * Returns closest point on a tetrahedron to a point.
	 * The idea is to identify the halfplanes that the point is
	 * in relative to each face of the tetrahedron
	 *
	 * @param	Point			point to check distance for
	 * @param	A,B,C,D			four points defining a tetrahedron
	 *
	 * @return	Point on tetrahedron ABCD closest to given point
	 */
	[[nodiscard]] static CORE_API FVector ClosestPointOnTetrahedronToPoint(const FVector& Point, const FVector& A, const FVector& B, const FVector& C, const FVector& D);

	/** 
	 * Find closest point on a Sphere to a Line.
	 * When line intersects		Sphere, then closest point to LineOrigin is returned.
	 * @param SphereOrigin		Origin of Sphere
	 * @param SphereRadius		Radius of Sphere
	 * @param LineOrigin		Origin of line
	 * @param LineDir			Direction of line. Needs to be normalized!!
	 * @param OutClosestPoint	Closest point on sphere to given line.
	 */
	static CORE_API void SphereDistToLine(FVector SphereOrigin, float SphereRadius, FVector LineOrigin, FVector LineDir, FVector& OutClosestPoint);

	/**
	 * Calculates whether a Point is within a cone segment, and also what percentage within the cone (100% is along the center line, whereas 0% is along the edge)
	 *
	 * @param Point - The Point in question
	 * @param ConeStartPoint - the beginning of the cone (with the smallest radius)
	 * @param ConeLine - the line out from the start point that ends at the largest radius point of the cone
	 * @param RadiusAtStart - the radius at the ConeStartPoint (0 for a 'proper' cone)
	 * @param RadiusAtEnd - the largest radius of the cone
	 * @param PercentageOut - output variable the holds how much within the cone the point is (1 = on center line, 0 = on exact edge or outside cone).
	 *
	 * @return true if the point is within the cone, false otherwise.
	 */
	[[nodiscard]] static CORE_API bool GetDistanceWithinConeSegment(FVector Point, FVector ConeStartPoint, FVector ConeLine, float RadiusAtStart, float RadiusAtEnd, float &PercentageOut);

	/**
	 * Determines whether a given set of points are coplanar, with a tolerance. Any three points or less are always coplanar.
	 *
	 * @param Points - The set of points to determine coplanarity for.
	 * @param Tolerance - Larger numbers means more variance is allowed.
	 *
	 * @return Whether the points are relatively coplanar, based on the tolerance
	 */
	[[nodiscard]] static CORE_API bool PointsAreCoplanar(const TArray<FVector>& Points, const float Tolerance = 0.1f);

	/**
	 * Truncates a floating point number to half if closer than the given tolerance.
	 * @param F					Floating point number to truncate
	 * @param Tolerance			Maximum allowed difference to 0.5 in order to truncate
	 * @return					The truncated value
	 */
	[[nodiscard]] static CORE_API float TruncateToHalfIfClose(float F, float Tolerance = UE_SMALL_NUMBER);
	[[nodiscard]] static CORE_API double TruncateToHalfIfClose(double F, double Tolerance = UE_SMALL_NUMBER);

	/**
	* Converts a floating point number to the nearest integer, equidistant ties go to the value which is closest to an even value: 1.5 becomes 2, 0.5 becomes 0
	* @param F		Floating point value to convert
	* @return		The rounded integer
	*/
	[[nodiscard]] static CORE_API float RoundHalfToEven(float F);
	[[nodiscard]] static CORE_API double RoundHalfToEven(double F);

	/**
	* Converts a floating point number to the nearest integer, equidistant ties go to the value which is further from zero: -0.5 becomes -1.0, 0.5 becomes 1.0
	* @param F		Floating point value to convert
	* @return		The rounded integer
	*/
	[[nodiscard]] static CORE_API float RoundHalfFromZero(float F);
	[[nodiscard]] static CORE_API double RoundHalfFromZero(double F);

	/**
	* Converts a floating point number to the nearest integer, equidistant ties go to the value which is closer to zero: -0.5 becomes 0, 0.5 becomes 0
	* @param F		Floating point value to convert
	* @return		The rounded integer
	*/
	[[nodiscard]] static CORE_API float RoundHalfToZero(float F);
	[[nodiscard]] static CORE_API double RoundHalfToZero(double F);

	/**
	* Converts a floating point number to an integer which is further from zero, "larger" in absolute value: 0.1 becomes 1, -0.1 becomes -1
	* @param F		Floating point value to convert
	* @return		The rounded integer
	*/
	[[nodiscard]] static UE_FORCEINLINE_HINT float RoundFromZero(float F)
	{
		return (F < 0.0f) ? FloorToFloat(F) : CeilToFloat(F);
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT double RoundFromZero(double F)
	{
		return (F < 0.0) ? FloorToDouble(F) : CeilToDouble(F);
	}

	/**
	* Converts a floating point number to an integer which is closer to zero, "smaller" in absolute value: 0.1 becomes 0, -0.1 becomes 0
	* @param F		Floating point value to convert
	* @return		The rounded integer
	*/
	[[nodiscard]] static UE_FORCEINLINE_HINT float RoundToZero(float F)
	{
		return (F < 0.0f) ? CeilToFloat(F) : FloorToFloat(F);
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT double RoundToZero(double F)
	{
		return (F < 0.0) ? CeilToDouble(F) : FloorToDouble(F);
	}

	/**
	* Converts a floating point number to an integer which is more negative: 0.1 becomes 0, -0.1 becomes -1
	* @param F		Floating point value to convert
	* @return		The rounded integer
	*/
	[[nodiscard]] static UE_FORCEINLINE_HINT float RoundToNegativeInfinity(float F)
	{
		return FloorToFloat(F);
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT double RoundToNegativeInfinity(double F)
	{
		return FloorToDouble(F);
	}

	/**
	* Converts a floating point number to an integer which is more positive: 0.1 becomes 1, -0.1 becomes 0
	* @param F		Floating point value to convert
	* @return		The rounded integer
	*/
	[[nodiscard]] static UE_FORCEINLINE_HINT float RoundToPositiveInfinity(float F)
	{
		return CeilToFloat(F);
	}

	[[nodiscard]] static UE_FORCEINLINE_HINT double RoundToPositiveInfinity(double F)
	{
		return CeilToDouble(F);
	}


	// Formatting functions

	/**
	 * Formats an integer value into a human readable string (i.e. 12345 becomes "12,345")
	 *
	 * @param	Val		The value to use
	 * @return	FString	The human readable string
	 */
	[[nodiscard]] static CORE_API FString FormatIntToHumanReadable(int32 Val);


	// Utilities

	/**
	 * Tests a memory region to see that it's working properly.
	 *
	 * @param BaseAddress	Starting address
	 * @param NumBytes		Number of bytes to test (will be rounded down to a multiple of 4)
	 * @return				true if the memory region passed the test
	 */
	[[nodiscard]] static CORE_API bool MemoryTest( void* BaseAddress, uint32 NumBytes );

	/**
	 * Evaluates a numerical equation.
	 *
	 * Operators and precedence: 1:+- 2:/% 3:* 4:^ 5:&|
	 * Unary: -
	 * Types: Numbers (0-9.), Hex ($0-$f)
	 * Grouping: ( )
	 *
	 * @param	Str			String containing the equation.
	 * @param	OutValue		Pointer to storage for the result.
	 * @return				1 if successful, 0 if equation fails.
	 */
	static CORE_API bool Eval( FString Str, float& OutValue );

	/**
	 * Computes the barycentric coordinates for a given point in a triangle, only considering the XY coordinates - simpler than the Compute versions
	 *
	 * @param	Point			point to convert to barycentric coordinates (in plane of ABC)
	 * @param	A,B,C			three non-collinear points defining a triangle in CCW
	 * 
	 * @return Vector containing the three weights a,b,c such that Point = a*A + b*B + c*C
	 *							                                or Point = A + b*(B-A) + c*(C-A) = (1-b-c)*A + b*B + c*C
	 */
	[[nodiscard]] static CORE_API FVector GetBaryCentric2D(const FVector& Point, const FVector& A, const FVector& B, const FVector& C);

	/**
	 * Computes the barycentric coordinates for a given point in a triangle - simpler than the Compute versions
	 *
	 * @param	Point			point to convert to barycentric coordinates (in plane of ABC)
	 * @param	A,B,C			three non-collinear points defining a triangle in CCW
	 *
	 * @return Vector containing the three weights a,b,c such that Point = a*A + b*B + c*C
	 *							                                or Point = A + b*(B-A) + c*(C-A) = (1-b-c)*A + b*B + c*C
	 */
	[[nodiscard]] static CORE_API FVector GetBaryCentric2D(const FVector2D& Point, const FVector2D& A, const FVector2D& B, const FVector2D& C);

	/**
	 * Computes the barycentric coordinates for a given point in a 3D triangle.
	 * Note: Prefer the more accurately-named ComputeBarycentricTri instead.
	 *
	 * @param	Point			point to convert to barycentric coordinates (in plane of ABC)
	 * @param	A,B,C			three non-collinear points defining a triangle in CCW
	 * 
	 * @return Vector containing the three weights a,b,c such that Point = a*A + b*B + c*C
	 *							                               or Point = A + b*(B-A) + c*(C-A) = (1-b-c)*A + b*B + c*C
	 */
	[[nodiscard]] static CORE_API FVector ComputeBaryCentric2D(const FVector& Point, const FVector& A, const FVector& B, const FVector& C);

	/**
	 * Computes the barycentric coordinates for a given point in a triangle
	 *
	 * @param	Point			point to convert to barycentric coordinates (in plane of ABC)
	 * @param	A,B,C			three non-collinear points defining a triangle in CCW
	 * @param	OutBarycentric	Vector containing the three weights a,b,c such that Point = a*A + b*B + c*C
	 *							                               or Point = A + b*(B-A) + c*(C-A) = (1-b-c)*A + b*B + c*C
	 * @param	Tolerance		Tolerance for ignoring too-small triangles
	 * 
	 * @return false if the result could not be computed (occurs when the triangle area is too small)
	 */
	[[nodiscard]] static CORE_API bool ComputeBarycentricTri(const FVector& Point, const FVector& A, const FVector& B, const FVector& C, FVector& OutBarycentric, double Tolerance = UE_DOUBLE_SMALL_NUMBER);

	/**
	 * Computes the barycentric coordinates for a given point on a tetrahedron (3D)
	 *
	 * @param	Point			point to convert to barycentric coordinates
	 * @param	A,B,C,D			four points defining a tetrahedron
	 *
	 * @return Vector containing the four weights a,b,c,d such that Point = a*A + b*B + c*C + d*D
	 */
	[[nodiscard]] static CORE_API FVector4 ComputeBaryCentric3D(const FVector& Point, const FVector& A, const FVector& B, const FVector& C, const FVector& D);

	/** 32 bit values where BitFlag[x] == (1<<x) */
	static CORE_API const uint32 BitFlag[32];

	/** 
	 * Returns a smooth Hermite interpolation between 0 and 1 for the value X (where X ranges between A and B)
	 * Clamped to 0 for X <= A and 1 for X >= B.
	 *
	 * @param A Minimum value of X
	 * @param B Maximum value of X
	 * @param X Parameter
	 *
	 * @return Smoothed value between 0 and 1
	 */
	template<typename T>
	[[nodiscard]] static constexpr T SmoothStep(T A, T B, T X)
	{
		if (X < A)
		{
			return 0;
		}
		else if (X >= B)
		{
			return 1;
		}
		const T InterpFraction = (X - A) / (B - A);
		return InterpFraction * InterpFraction * (3.0f - 2.0f * InterpFraction);
	}

	/**
	 * Get a bit in memory created from bitflags (uint32 Value:1), used for EngineShowFlags,
	 * TestBitFieldFunctions() tests the implementation
	 */
	[[nodiscard]] static constexpr bool ExtractBoolFromBitfield(const uint8* Ptr, uint32 Index)
	{
		const uint8* BytePtr = Ptr + Index / 8;
		uint8 Mask = (uint8)(1 << (Index & 0x7));

		return (*BytePtr & Mask) != 0;
	}

	/**
	 * Set a bit in memory created from bitflags (uint32 Value:1), used for EngineShowFlags,
	 * TestBitFieldFunctions() tests the implementation
	 */
	static constexpr void SetBoolInBitField(uint8* Ptr, uint32 Index, bool bSet)
	{
		uint8* BytePtr = Ptr + Index / 8;
		uint8 Mask = (uint8)(1 << (Index & 0x7));

		if(bSet)
		{
			*BytePtr |= Mask;
		}
		else
		{
			*BytePtr &= ~Mask;
		}
	}

	/**
	 * Handy to apply scaling in the editor
	 * @param Dst in and out
	 */
	static CORE_API void ApplyScaleToFloat(float& Dst, const FVector& DeltaScale, float Magnitude = 1.0f);

	// @param x assumed to be in this range: 0..1
	// @return 0..255
	[[nodiscard]] static uint8 Quantize8UnsignedByte(float x)
	{
		// 0..1 -> 0..255
		int32 Ret = (int32)(x * 255.f + 0.5f);

		check(Ret >= 0);
		check(Ret <= 255);

		return (uint8)Ret;
	}
	
	// @param x assumed to be in this range: -1..1
	// @return 0..255
	[[nodiscard]] static uint8 Quantize8SignedByte(float x)
	{
		// -1..1 -> 0..1
		float y = x * 0.5f + 0.5f;

		return Quantize8UnsignedByte(y);
	}

	// Use the Euclidean method to find the GCD
	[[nodiscard]] static constexpr int32 GreatestCommonDivisor(int32 a, int32 b)
	{
		while (b != 0)
		{
			int32 t = b;
			b = a % b;
			a = t;
		}
		return a;
	}

	// LCM = a/gcd * b
	// a and b are the number we want to find the lcm
	[[nodiscard]] static constexpr int32 LeastCommonMultiplier(int32 a, int32 b)
	{
		int32 CurrentGcd = GreatestCommonDivisor(a, b);
		return CurrentGcd == 0 ? 0 : (a / CurrentGcd) * b;
	}

	/**
	 * Generates a 1D Perlin noise from the given value.  Returns a continuous random value between -1.0 and 1.0.
	 *
	 * @param	Value	The input value that Perlin noise will be generated from.  This is usually a steadily incrementing time value.
	 *
	 * @return	Perlin noise in the range of -1.0 to 1.0
	 */
	[[nodiscard]] static CORE_API float PerlinNoise1D(float Value);

	/**
	* Generates a 2D Perlin noise sample at the given location.  Returns a continuous random value between -1.0 and 1.0.
	*
	* @param	Location	Where to sample
	*
	* @return	Perlin noise in the range of -1.0 to 1.0
	*/
	[[nodiscard]] static CORE_API float PerlinNoise2D(const FVector2D& Location);
	 

	/**
	* Generates a 3D Perlin noise sample at the given location.  Returns a continuous random value between -1.0 and 1.0.
	*
	* @param	Location	Where to sample
	*
	* @return	Perlin noise in the range of -1.0 to 1.0
	*/
	[[nodiscard]] static CORE_API float PerlinNoise3D(const FVector& Location);

	/**
	 * Calculates the new value in a weighted moving average series using the previous value and the weight
	 *
	 * @param CurrentSample - The value to blend with the previous sample to get a new weighted value
	 * @param PreviousSample - The last value from the series
	 * @param Weight - The weight to blend with
	 *
	 * @return the next value in the series
	 */
	template<typename T>
	[[nodiscard]] static inline T WeightedMovingAverage(T CurrentSample, T PreviousSample, T Weight)
	{
		Weight = Clamp<T>(Weight, 0.f, 1.f);
		T WAvg = (CurrentSample * Weight) + (PreviousSample * (1.f - Weight));
		return WAvg;
	}

	/**
	 * Calculates the new value in a weighted moving average series using the previous value and a weight range.
	 * The weight range is used to dynamically adjust based upon distance between the samples
	 * This allows you to smooth a value more aggressively for small noise and let large movements be smoothed less (or vice versa)
	 *
	 * @param CurrentSample - The value to blend with the previous sample to get a new weighted value
	 * @param PreviousSample - The last value from the series
	 * @param MaxDistance - Distance to use as the blend between min weight or max weight
	 * @param MinWeight - The weight use when the distance is small
	 * @param MaxWeight - The weight use when the distance is large
	 *
	 * @return the next value in the series
	 */
	template<typename T>
	[[nodiscard]] static inline T DynamicWeightedMovingAverage(T CurrentSample, T PreviousSample, T MaxDistance, T MinWeight, T MaxWeight)
	{
		// We need the distance between samples to determine how much of each weight to use
		const T Distance = Abs<T>(CurrentSample - PreviousSample);
		T Weight = MinWeight;
		if (MaxDistance > 0)
		{
			// Figure out the lerp value to use between the min/max weights
			const T LerpAlpha = Clamp<T>(Distance / MaxDistance, 0.f, 1.f);
			Weight = Lerp<T>(MinWeight, MaxWeight, LerpAlpha);
		}
		return WeightedMovingAverage(CurrentSample, PreviousSample, Weight);
	}
	
	/**
	 * Calculate the inverse of an FMatrix44.  Src == Dst is allowed
	 *
	 * @param DstMatrix		FMatrix44 pointer to where the result should be stored
	 * @param SrcMatrix		FMatrix44 pointer to the Matrix to be inversed
	 * @return bool			returns false if matrix is not invertable and stores identity 
	 *
	 * Do not call this directly, use VectorMatrixInverse or Matrix::Inverse
	 * this is the fallback scalar implementation used by VectorMatrixInverse
	 */
	[[nodiscard]] static CORE_API bool MatrixInverse(FMatrix44f* DstMatrix, const FMatrix44f* SrcMatrix);
	[[nodiscard]] static CORE_API bool MatrixInverse(FMatrix44d* DstMatrix, const FMatrix44d* SrcMatrix);

};

// LWC Conversion helpers
namespace UE
{
namespace LWC
{

	// Convert array to a new type
	template<typename TDest, typename TSrc, typename InAllocatorType>
	TArray<TDest, InAllocatorType> ConvertArrayType(const TArray<TSrc, InAllocatorType>& From)
	{
		//static_assert(!std::is_same_v<TDest, TSrc>, "Redundant call to ConvertArrayType");	// Unavoidable if supporting LWC toggle, but a useful check once LWC is locked to enabled.
		if constexpr (std::is_same_v<TDest, TSrc>)
		{
			return From;
		}
		else
		{
			TArray<TDest, InAllocatorType> Converted;
			Converted.Reserve(From.Num());
			for (const TSrc& Item : From)
			{
				Converted.Add(static_cast<TDest>(Item));
			}
			return Converted;
		}
	}

	// Convert array to a new type and clamps values to the Max of TDest type
	template<typename TDest, typename TSrc, typename InAllocatorType>
	TArray<TDest, InAllocatorType> ConvertArrayTypeClampMax(const TArray<TSrc, InAllocatorType>& From)
	{
		//static_assert(!std::is_same_v<TDest, TSrc>, "Redundant call to ConvertArrayType");	// Unavoidable if supporting LWC toggle, but a useful check once LWC is locked to enabled.
		if constexpr (std::is_same_v<TDest, TSrc>)
		{
			return From;
		}
		else
		{
			TArray<TDest, InAllocatorType> Converted;
			Converted.Reserve(From.Num());
			for (const TSrc& Item : From)
			{
				Converted.Add(FMath::Min(TNumericLimits<TDest>::Max(), static_cast<TDest>(Item)));
			}
			return Converted;
		}
	}

	/*
	 * Floating point to integer conversions
	 */

	// Generic float type to int type, to enable specializations below.
	template <UE::CIntegral OutIntType, UE::CFloatingPoint InFloatType>
	inline OutIntType FloatToIntCastChecked(InFloatType FloatValue)
	{
		return (OutIntType)(FloatValue);
	}

	// float->int32
	template<>
	inline int32 FloatToIntCastChecked(float FloatValue)
	{
		// floats over 2^31 - 1 - 64 overflow int32 after conversion.
		checkf(FloatValue >= float(TNumericLimits<int32>::Lowest()) && FloatValue <= float(TNumericLimits<int32>::Max() - 64), TEXT("Input value %f will exceed int32 limits"), FloatValue);
		return FMath::TruncToInt32(FloatValue);
	}

	// float->int64
	template<>
	inline int64 FloatToIntCastChecked(float FloatValue)
	{
		// floats over 2^63 - 1 - 2^39 overflow int64 after conversion.
		checkf(FloatValue >= float(TNumericLimits<int64>::Lowest()) && FloatValue <= float(TNumericLimits<int64>::Max() - (int64)549755813888), TEXT("Input value %f will exceed int64 limits"), FloatValue);
		return FMath::TruncToInt64(FloatValue);
	}

	// double->int32
	template<>
	inline int32 FloatToIntCastChecked(double FloatValue)
	{
		checkf(FloatValue >= double(TNumericLimits<int32>::Lowest()) && FloatValue <= double(TNumericLimits<int32>::Max()), TEXT("Input value %f will exceed int32 limits"), FloatValue);
		return FMath::TruncToInt32(FloatValue);
	}

	// double->int64
	template<>
	inline int64 FloatToIntCastChecked(double FloatValue)
	{
		// doubles over 2^63 - 1 - 512 overflow int64 after conversion.
		checkf(FloatValue >= double(TNumericLimits<int64>::Lowest()) && FloatValue <= double(TNumericLimits<int64>::Max() - 512), TEXT("Input value %f will exceed int64 limits"), FloatValue);
		return FMath::TruncToInt64(FloatValue);
	}

} // namespace LWC
} // namespace UE

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Templates/Identity.h"
#endif
