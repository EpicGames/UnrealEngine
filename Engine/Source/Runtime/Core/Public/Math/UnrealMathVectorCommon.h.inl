// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

struct FGenericPlatformMath;
struct FMath;

// Overload to resolve compiler ambiguity for things like MakeVectorRegister(V.X, V.Y, V.Z, 0.f) when V is a double type.
FORCEINLINE VectorRegister4Double MakeVectorRegister(double X, double Y, double Z, float W)
{
	return MakeVectorRegisterDouble(X, Y, Z, (double)W);
}

FORCEINLINE VectorRegister4Float VectorZero(void)
{
	return VectorZeroFloat();
}

FORCEINLINE VectorRegister4Float VectorOne(void)
{
	return VectorOneFloat();
}

/**
 * Overloads for code passing &MyVector, &MyQuat, etc.
 */

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/// VectorLoad
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename T>
FORCEINLINE TVectorRegisterType<T> VectorLoad(const UE::Math::TQuat<T>* Ptr)
{
	return VectorLoad((const T*)(Ptr));
}

template<typename T>
FORCEINLINE TVectorRegisterType<T> VectorLoad(const UE::Math::TVector4<T>* Ptr)
{
	return VectorLoad((const T*)(Ptr));
}

FORCEINLINE VectorRegister4Float VectorLoad(const VectorRegister4Float* Ptr)
{
	return VectorLoad((const float*)(Ptr));
}

FORCEINLINE VectorRegister4Double VectorLoad(const VectorRegister4Double* Ptr)
{
	return VectorLoad((const double*)(Ptr));
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/// VectorLoadAligned
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename T>
FORCEINLINE TVectorRegisterType<T> VectorLoadAligned(const UE::Math::TQuat<T>* Ptr)
{
	return VectorLoadAligned((const T*)(Ptr));
}

template<typename T>
FORCEINLINE TVectorRegisterType<T> VectorLoadAligned(const UE::Math::TVector4<T>* Ptr)
{
	return VectorLoadAligned((const T*)(Ptr));
}

template<typename T>
FORCEINLINE TVectorRegisterType<T> VectorLoadAligned(const UE::Math::TPlane<T>* Ptr)
{
	return VectorLoadAligned((const T*)(Ptr));
}

FORCEINLINE VectorRegister4Float VectorLoadAligned(const VectorRegister4Float* Ptr)
{
	//return VectorLoadAligned((const float*)(Ptr));
	return *Ptr;
}

FORCEINLINE VectorRegister4Double VectorLoadAligned(const VectorRegister4Double* Ptr)
{
	//return VectorLoadAligned((const double*)(Ptr));
	return *Ptr;
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/// VectorLoadFloat3
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
* Loads 3 floats from unaligned memory and sets W=0.
*
* @param Ptr	Unaligned memory pointer to the 3 floats
* @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[2], 0.0f)
*/
FORCEINLINE VectorRegister4Float VectorLoadFloat3(const float* Ptr)
{
	return MakeVectorRegister(Ptr[0], Ptr[1], Ptr[2], 0.0f);
}

template<typename T>
FORCEINLINE TVectorRegisterType<T> VectorLoadFloat3(const UE::Math::TVector<T>* Ptr)
{
	return VectorLoadFloat3((const T*)(Ptr));
}

FORCEINLINE VectorRegister4Double VectorLoadDouble3(const double* Ptr)
{
	return VectorLoadFloat3((const double*)Ptr);
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/// VectorLoadFloat3_W0
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Loads 3 floats from unaligned memory and sets W=0.
 *
 * @param Ptr	Unaligned memory pointer to the 3 floats
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[2], 0.0f)
 */
FORCEINLINE VectorRegister4Float VectorLoadFloat3_W0(const float* Ptr)
{
	return VectorLoadFloat3(Ptr);
}

FORCEINLINE VectorRegister4Double VectorLoadFloat3_W0(const double* Ptr)
{
	return VectorLoadFloat3(Ptr);
}

template<typename T>
FORCEINLINE TVectorRegisterType<T> VectorLoadFloat3_W0(const UE::Math::TVector<T>* Ptr)
{
	return VectorLoadFloat3_W0((const T*)(Ptr));
}

template<typename T>
FORCEINLINE TVectorRegisterType<T> VectorLoadFloat3_W0(const UE::Math::TRotator<T>* Ptr)
{
	return VectorLoadFloat3_W0((const T*)(Ptr));
}

FORCEINLINE VectorRegister4Double VectorLoadDouble3_W0(const double* Ptr)
{
	return VectorLoadDouble3(Ptr);
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/// VectorLoadFloat3_W1
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Loads 3 floats from unaligned memory and sets W=1.
 *
 * @param Ptr	Unaligned memory pointer to the 3 floats
 * @return		VectorRegister4Float(Ptr[0], Ptr[1], Ptr[2], 1.0f)
 */
FORCEINLINE VectorRegister4Float VectorLoadFloat3_W1(const float* Ptr)
{
	return MakeVectorRegister(Ptr[0], Ptr[1], Ptr[2], 1.0f);
}

template<typename T>
FORCEINLINE TVectorRegisterType<T> VectorLoadFloat3_W1(const UE::Math::TVector<T>* Ptr)
{
	return VectorLoadFloat3_W1((const T*)Ptr);
}

FORCEINLINE VectorRegister4Double VectorLoadDouble3_W1(const double* Ptr)
{
	return VectorLoadFloat3_W1(Ptr);
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/// VectorLoadFloat1
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FORCEINLINE VectorRegister4Float VectorLoadFloat1(const VectorRegister4Float* Ptr)
{
	return VectorReplicate(*Ptr, 0);
}

FORCEINLINE VectorRegister4Double VectorLoadFloat1(const VectorRegister4Double* Ptr)
{
	return VectorReplicate(*Ptr, 0);
}

FORCEINLINE VectorRegister4Double VectorLoadFloat1(const double* Ptr)
{
	return VectorLoadDouble1(Ptr);
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/// VectorStoreAligned
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////

// TODO: this is lossy, would rather not implement.
#if SUPPORT_DOUBLE_TO_FLOAT_VECTOR_CONVERSION
FORCEINLINE void VectorStoreAligned(VectorRegister4Double Vec, float* Dst)
{
	VectorStoreAligned(MakeVectorRegisterFloatFromDouble(Vec), Dst);
}
#endif

template<typename T>
FORCEINLINE void VectorStoreAligned(TVectorRegisterType<T> Vec, UE::Math::TVector4<T>* Dst)
{
	VectorStoreAligned(Vec, (T*)Dst);
}

// Specific overload to support promoting float->double vector and storing in TVector4<double>
FORCEINLINE void VectorStoreAligned(VectorRegister4Float Vec, struct UE::Math::TVector4<double>* Dst)
{
	VectorRegister4Double DoubleVec(Vec);
	VectorStoreAligned(DoubleVec, Dst);
}

template<typename T>
FORCEINLINE void VectorStoreAligned(TVectorRegisterType<T> Vec, struct UE::Math::TQuat<T>* Dst)
{
	VectorStoreAligned(Vec, (T*)Dst);
}

// Specific overload to support promoting float->double vector and storing in TQuat<double>
FORCEINLINE void VectorStoreAligned(VectorRegister4Float Vec, struct UE::Math::TQuat<double>* Dst)
{
	VectorRegister4Double DoubleVec(Vec);
	VectorStoreAligned(DoubleVec, Dst);
}

FORCEINLINE void VectorStoreAligned(VectorRegister4Float Vec, VectorRegister4Float* Dst)
{
	*Dst = Vec;
}

FORCEINLINE void VectorStoreAligned(VectorRegister4Double Vec, VectorRegister4Double* Dst)
{
	*Dst = Vec;
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/// VectorStore
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SUPPORT_DOUBLE_TO_FLOAT_VECTOR_CONVERSION
FORCEINLINE void VectorStore(VectorRegister4Double Vec, float* Dst)
{
	VectorRegister4Float FloatVec = MakeVectorRegisterFloatFromDouble(Vec);
	VectorStore(FloatVec, Dst);
}
#endif

template<typename T>
FORCEINLINE void VectorStore(VectorRegister4Float Vec, UE::Math::TVector4<T>* Dst)
{
	VectorStoreAligned(Vec, Dst);
}

template<typename T>
FORCEINLINE void VectorStore(VectorRegister4Double Vec, UE::Math::TVector4<T>* Dst)
{
	VectorStoreAligned(Vec, Dst);
}

template<typename T>
FORCEINLINE void VectorStore(TVectorRegisterType<T> Vec, struct UE::Math::TQuat<T>* Dst)
{
	VectorStoreAligned(Vec, Dst);
}

// Specific overload to support promoting float->double vector and storing in TQuat<double>
FORCEINLINE void VectorStore(VectorRegister4Float Vec, struct UE::Math::TQuat<double>* Dst)
{
	VectorRegister4Double DoubleVec(Vec);
	VectorStoreAligned(DoubleVec, Dst);
}

FORCEINLINE void VectorStore(VectorRegister4Float Vec, VectorRegister4Float* Dst)
{
	*Dst = Vec;
}

FORCEINLINE void VectorStore(VectorRegister4Double Vec, VectorRegister4Double* Dst)
{
	*Dst = Vec;
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/// VectorStoreFloat3
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SUPPORT_DOUBLE_TO_FLOAT_VECTOR_CONVERSION
FORCEINLINE void VectorStoreFloat3(VectorRegister4Double Vec, float* Dst)
{
	VectorRegister4Float FloatVec = MakeVectorRegisterFloatFromDouble(Vec);
	VectorStoreFloat3(FloatVec, Dst);
}
#endif

template<typename T>
FORCEINLINE void VectorStoreFloat3(VectorRegister4Float Vec, UE::Math::TVector<T>* Dst)
{
	VectorStoreFloat3(Vec, (T*)Dst);
}

template<typename T>
FORCEINLINE void VectorStoreFloat3(VectorRegister4Double Vec, UE::Math::TVector<T>* Dst)
{
	VectorStoreFloat3(Vec, (T*)Dst);
}

template<typename T>
FORCEINLINE void VectorStoreFloat3(VectorRegister4Float Vec, UE::Math::TRotator<T>* Dst)
{
	VectorStoreFloat3(Vec, (T*)Dst);
}

template<typename T>
FORCEINLINE void VectorStoreFloat3(VectorRegister4Double Vec, UE::Math::TRotator<T>* Dst)
{
	VectorStoreFloat3(Vec, (T*)Dst);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/// VectorStoreFloat1
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SUPPORT_DOUBLE_TO_FLOAT_VECTOR_CONVERSION
FORCEINLINE void VectorStoreFloat1(VectorRegister4Double Vec, float* Dst)
{
	VectorRegister4Float FloatVec = MakeVectorRegisterFloatFromDouble(Vec);
	VectorStoreFloat1(FloatVec, Dst);
}
#endif


FORCEINLINE void VectorStoreFloat1(VectorRegister4Float Vec, int32* Dst)
{
	VectorStoreFloat1(Vec, (float*)Dst);
}

FORCEINLINE void VectorStoreFloat1(VectorRegister4Double Vec, int32* Dst)
{
	// This is used for bit masks, so don't do an actual convert, mask the lower bits instead
	double X = VectorGetComponent(Vec, 0);
	uint64 XMask64 = *((uint64*)(&X));
	uint32 XMask32 = (uint32)(XMask64 & 0xFFFFFFFF);
	VectorRegister4Float FloatVec = VectorLoadFloat1((float*)(&XMask32));
	VectorStoreFloat1(FloatVec, Dst);
}

FORCEINLINE void VectorStoreFloat1(VectorRegister4Double Vec, int64* Dst)
{
	VectorStoreFloat1(Vec, (double*)Dst);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Wrappers for clarifying "accurate" versus "estimate" for some functions,
// important for old functions which were ambiguous in their precision.

// Returns accurate reciprocal square root.
FORCEINLINE VectorRegister4Float VectorReciprocalSqrtAccurate(VectorRegister4Float Vec)
{
	return VectorReciprocalSqrt(Vec);
}

FORCEINLINE VectorRegister4Double VectorReciprocalSqrtAccurate(VectorRegister4Double Vec)
{
	return VectorReciprocalSqrt(Vec);
}

/**
 * Computes the reciprocal of a vector (component-wise) and returns the result.
 *
 * @param Vec	1st vector
 * @return		VectorRegister( 1.0f / Vec.x, 1.0f / Vec.y, 1.0f / Vec.z, 1.0f / Vec.w )
 */
FORCEINLINE VectorRegister4Float VectorReciprocalAccurate(VectorRegister4Float Vec)
{
	return VectorReciprocal(Vec);
}

FORCEINLINE VectorRegister4Double VectorReciprocalAccurate(VectorRegister4Double Vec)
{
	return VectorReciprocal(Vec);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////

// Returns a normalized 4 vector = Vector / |Vector|.
// There is no handling of zero length vectors, use VectorNormalizeSafe if this is a possible input.
template<typename TVectorRegisterType>
FORCEINLINE TVectorRegisterType VectorNormalize(TVectorRegisterType Vector)
{
	return VectorMultiply(Vector, VectorReciprocalLen(Vector));
}

template<typename TVectorRegisterType>
FORCEINLINE TVectorRegisterType VectorNormalizeAccurate(TVectorRegisterType Vector)
{
	return VectorNormalize(Vector);
}

template<typename TVectorRegisterType>
FORCEINLINE TVectorRegisterType VectorNormalizeEstimate(TVectorRegisterType Vector)
{
	return VectorMultiply(Vector, VectorReciprocalLenEstimate(Vector));
}

// Returns ((Vector dot Vector) >= 1e-8) ? (Vector / |Vector|) : DefaultValue
// Uses accurate 1/sqrt, not the estimate
FORCEINLINE VectorRegister4Float VectorNormalizeSafe(VectorRegister4Float Vector, VectorRegister4Float DefaultValue)
{
	const VectorRegister4Float SquareSum = VectorDot4(Vector, Vector);
	const VectorRegister4Float NonZeroMask = VectorCompareGE(SquareSum, GlobalVectorConstants::SmallLengthThreshold);
	const VectorRegister4Float InvLength = VectorReciprocalSqrtAccurate(SquareSum);
	const VectorRegister4Float NormalizedVector = VectorMultiply(InvLength, Vector);
	return VectorSelect(NonZeroMask, NormalizedVector, DefaultValue);
}

FORCEINLINE VectorRegister4Double VectorNormalizeSafe(VectorRegister4Double Vector, VectorRegister4Double DefaultValue)
{
	const VectorRegister4Double SquareSum = VectorDot4(Vector, Vector);
	const VectorRegister4Double NonZeroMask = VectorCompareGE(SquareSum, GlobalVectorConstants::DoubleSmallLengthThreshold);
	const VectorRegister4Double InvLength = VectorReciprocalSqrtAccurate(SquareSum);
	const VectorRegister4Double NormalizedVector = VectorMultiply(InvLength, Vector);
	return VectorSelect(NonZeroMask, NormalizedVector, DefaultValue);
}

/**
 * Returns non-zero if any element in Vec1 is lesser than the corresponding element in Vec2, otherwise 0.
 *
 * @param Vec1			1st source vector
 * @param Vec2			2nd source vector
 * @return				Non-zero integer if (Vec1.x < Vec2.x) || (Vec1.y < Vec2.y) || (Vec1.z < Vec2.z) || (Vec1.w < Vec2.w)
 */
FORCEINLINE uint32 VectorAnyLesserThan(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return VectorAnyGreaterThan( Vec2, Vec1 );
}

FORCEINLINE uint32 VectorAnyLesserThan(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	return VectorAnyGreaterThan(Vec2, Vec1);
}

/**
 * Returns non-zero if all elements in Vec1 are greater than the corresponding elements in Vec2, otherwise 0.
 *
 * @param Vec1			1st source vector
 * @param Vec2			2nd source vector
 * @return				Non-zero integer if (Vec1.x > Vec2.x) && (Vec1.y > Vec2.y) && (Vec1.z > Vec2.z) && (Vec1.w > Vec2.w)
 */
FORCEINLINE uint32 VectorAllGreaterThan(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return !VectorAnyGreaterThan( Vec2, Vec1 );
}

FORCEINLINE uint32 VectorAllGreaterThan(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	return !VectorAnyGreaterThan(Vec2, Vec1);
}

/**
 * Returns non-zero if all elements in Vec1 are lesser than the corresponding elements in Vec2, otherwise 0.
 *
 * @param Vec1			1st source vector
 * @param Vec2			2nd source vector
 * @return				Non-zero integer if (Vec1.x < Vec2.x) && (Vec1.y < Vec2.y) && (Vec1.z < Vec2.z) && (Vec1.w < Vec2.w)
 */
FORCEINLINE uint32 VectorAllLesserThan(VectorRegister4Float Vec1, VectorRegister4Float Vec2)
{
	return !VectorAnyGreaterThan( Vec1, Vec2 );
}

FORCEINLINE uint32 VectorAllLesserThan(VectorRegister4Double Vec1, VectorRegister4Double Vec2)
{
	return !VectorAnyGreaterThan(Vec1, Vec2);
}

/** Clamps X to be between VecMin and VecMax, inclusive. */
FORCEINLINE VectorRegister4Float VectorClamp(VectorRegister4Float X, VectorRegister4Float VecMin, VectorRegister4Float VecMax)
{
	return VectorMin(VectorMax(X, VecMin), VecMax);
}

FORCEINLINE VectorRegister4Double VectorClamp(VectorRegister4Double X, VectorRegister4Double VecMin, VectorRegister4Double VecMax)
{
	return VectorMin(VectorMax(X, VecMin), VecMax);
}

/*----------------------------------------------------------------------------
	VectorRegister specialization of templates.
----------------------------------------------------------------------------*/

/** Returns the smaller of the two values (operates on each component individually) */
template<> FORCEINLINE VectorRegister4Float FGenericPlatformMath::Min(VectorRegister4Float A, VectorRegister4Float B)
{
	return VectorMin(A, B);
}

template<> FORCEINLINE VectorRegister4Double FGenericPlatformMath::Min(VectorRegister4Double A, VectorRegister4Double B)
{
	return VectorMin(A, B);
}

/** Returns the larger of the two values (operates on each component individually) */
template<> FORCEINLINE VectorRegister4Float FGenericPlatformMath::Max(VectorRegister4Float A, VectorRegister4Float B)
{
	return VectorMax(A, B);
}

template<> FORCEINLINE VectorRegister4Double FGenericPlatformMath::Max(VectorRegister4Double A, VectorRegister4Double B)
{
	return VectorMax(A, B);
}

/** Lerp between two vectors */
FORCEINLINE VectorRegister4Float VectorLerp(VectorRegister4Float A, VectorRegister4Float B, VectorRegister4Float Alpha)
{
	VectorRegister4Float SubVec = VectorSubtract(GlobalVectorConstants::FloatOne, Alpha);
	return VectorMultiplyAdd(B, Alpha, VectorMultiply(A, SubVec));
}

FORCEINLINE VectorRegister4Double VectorLerp(VectorRegister4Double A, VectorRegister4Double B, VectorRegister4Double Alpha)
{
	VectorRegister4Double SubVec = VectorSubtract(GlobalVectorConstants::DoubleOne, Alpha);
	return VectorMultiplyAdd(B, Alpha, VectorMultiply(A, SubVec));
}

// TCustomLerp for FMath::Lerp<VectorRegister4Float>()
template<>
struct TCustomLerp<VectorRegister4Float>
{
	constexpr static bool Value = true;
	
	// Specialization of Lerp function that works with vector registers
	static FORCEINLINE VectorRegister4Float Lerp(VectorRegister4Float A, VectorRegister4Float B, VectorRegister4Float Alpha)
	{
		return VectorLerp(A, B, Alpha);
	}
};

// TCustomLerp for FMath::Lerp<VectorRegister4Double>()
template<>
struct TCustomLerp<VectorRegister4Double>
{
	constexpr static bool Value = true;

	// Specialization of Lerp function that works with vector registers
	static FORCEINLINE VectorRegister4Double Lerp(VectorRegister4Double A, VectorRegister4Double B, VectorRegister4Double Alpha)
	{
		return VectorLerp(A, B, Alpha);
	}
};

// A and B are quaternions.  The result is A + (|A.B| >= 0 ? 1 : -1) * B
FORCEINLINE VectorRegister4Float VectorAccumulateQuaternionShortestPath(VectorRegister4Float A, VectorRegister4Float B)
{
	// Blend rotation
	//     To ensure the 'shortest route', we make sure the dot product between the both rotations is positive.
	//     const float Bias = (|A.B| >= 0 ? 1 : -1)
	//     return A + B * Bias;
	const VectorRegister4Float Zero = VectorZeroFloat();
	const VectorRegister4Float RotationDot = VectorDot4(A, B);
	const VectorRegister4Float QuatRotationDirMask = VectorCompareGE(RotationDot, Zero);
	const VectorRegister4Float NegativeB = VectorSubtract(Zero, B);
	const VectorRegister4Float BiasTimesB = VectorSelect(QuatRotationDirMask, B, NegativeB);
	return VectorAdd(A, BiasTimesB);
}

FORCEINLINE VectorRegister4Double VectorAccumulateQuaternionShortestPath(VectorRegister4Double A, VectorRegister4Double B)
{
	// Blend rotation
	//     To ensure the 'shortest route', we make sure the dot product between the both rotations is positive.
	//     const float Bias = (|A.B| >= 0 ? 1 : -1)
	//     return A + B * Bias;
	const VectorRegister4Double Zero = VectorZeroDouble();
	const VectorRegister4Double RotationDot = VectorDot4(A, B);
	const VectorRegister4Double QuatRotationDirMask = VectorCompareGE(RotationDot, Zero);
	const VectorRegister4Double NegativeB = VectorSubtract(Zero, B);
	const VectorRegister4Double BiasTimesB = VectorSelect(QuatRotationDirMask, B, NegativeB);
	return VectorAdd(A, BiasTimesB);
}

// Normalize quaternion ( result = (Q.Q >= 1e-8) ? (Q / |Q|) : (0,0,0,1) )
FORCEINLINE VectorRegister4Float VectorNormalizeQuaternion(VectorRegister4Float UnnormalizedQuat)
{
	return VectorNormalizeSafe(UnnormalizedQuat, GlobalVectorConstants::Float0001);
}

FORCEINLINE VectorRegister4Double VectorNormalizeQuaternion(VectorRegister4Double UnnormalizedQuat)
{
	return VectorNormalizeSafe(UnnormalizedQuat, GlobalVectorConstants::Double0001);
}

// VectorMod360: Essentially VectorMod(X, 360) but using faster computation that is still accurate given the known input constraints.
FORCEINLINE VectorRegister4Float VectorMod360(VectorRegister4Float X)
{
	// R = X - (Y * (Trunc(X / Y))
	VectorRegister4Float Temp = VectorTruncate(VectorDivide(X, GlobalVectorConstants::Float360));
	VectorRegister4Float FloatResult = VectorNegateMultiplyAdd(GlobalVectorConstants::Float360, Temp, X);
	return FloatResult;
}

FORCEINLINE VectorRegister4Double VectorMod360(VectorRegister4Double X)
{
	// R = X - (Y * (Trunc(X / Y))
	VectorRegister4Double Temp = VectorTruncate(VectorDivide(X, GlobalVectorConstants::Double360));
	VectorRegister4Double DoubleResult = VectorNegateMultiplyAdd(GlobalVectorConstants::Double360, Temp, X);
	return DoubleResult;
}

// Normalize Rotator
FORCEINLINE VectorRegister4Float VectorNormalizeRotator(VectorRegister4Float UnnormalizedRotator)
{
	// shift in the range [-360,360]
	VectorRegister4Float V0	= VectorMod360(UnnormalizedRotator);
	VectorRegister4Float V1	= VectorAdd( V0, GlobalVectorConstants::Float360 );
	VectorRegister4Float V2	= VectorSelect(VectorCompareGE(V0, VectorZeroFloat()), V0, V1);

	// shift to [-180,180]
	VectorRegister4Float V3	= VectorSubtract( V2, GlobalVectorConstants::Float360 );
	VectorRegister4Float V4	= VectorSelect(VectorCompareGT(V2, GlobalVectorConstants::Float180), V3, V2);

	return  V4;
}

FORCEINLINE VectorRegister4Double VectorNormalizeRotator(VectorRegister4Double UnnormalizedRotator)
{
	// shift in the range [-360,360]
	VectorRegister4Double V0 = VectorMod360(UnnormalizedRotator);
	VectorRegister4Double V1 = VectorAdd(V0, GlobalVectorConstants::Double360);
	VectorRegister4Double V2 = VectorSelect(VectorCompareGE(V0, VectorZeroDouble()), V0, V1);

	// shift to [-180,180]
	VectorRegister4Double V3 = VectorSubtract(V2, GlobalVectorConstants::Double360);
	VectorRegister4Double V4 = VectorSelect(VectorCompareGT(V2, GlobalVectorConstants::Double180), V3, V2);

	return  V4;
}

/** 
 * Fast Linear Quaternion Interpolation for quaternions stored in VectorRegisters.
 * Result is NOT normalized.
 */
FORCEINLINE VectorRegister4Float VectorLerpQuat(VectorRegister4Float A, VectorRegister4Float B, VectorRegister4Float Alpha)
{
	// Blend rotation
	//     To ensure the 'shortest route', we make sure the dot product between the both rotations is positive.
	//     const float Bias = (|A.B| >= 0 ? 1 : -1)
	//     Rotation = (B * Alpha) + (A * (Bias * (1.f - Alpha)));
	const VectorRegister4Float Zero = VectorZeroFloat();

	const VectorRegister4Float OneMinusAlpha = VectorSubtract(VectorOneFloat(), Alpha);

	const VectorRegister4Float RotationDot = VectorDot4(A, B);
	const VectorRegister4Float QuatRotationDirMask = VectorCompareGE(RotationDot, Zero);
	const VectorRegister4Float NegativeA = VectorSubtract(Zero, A);
	const VectorRegister4Float BiasTimesA = VectorSelect(QuatRotationDirMask, A, NegativeA);
	const VectorRegister4Float BTimesWeight = VectorMultiply(B, Alpha);
	const VectorRegister4Float UnnormalizedResult = VectorMultiplyAdd(BiasTimesA, OneMinusAlpha, BTimesWeight);

	return UnnormalizedResult;
}

FORCEINLINE VectorRegister4Double VectorLerpQuat(VectorRegister4Double A, VectorRegister4Double B, VectorRegister4Double Alpha)
{
	// Blend rotation
	//     To ensure the 'shortest route', we make sure the dot product between the both rotations is positive.
	//     const float Bias = (|A.B| >= 0 ? 1 : -1)
	//     Rotation = (B * Alpha) + (A * (Bias * (1.f - Alpha)));
	const VectorRegister4Double Zero = VectorZeroDouble();

	const VectorRegister4Double OneMinusAlpha = VectorSubtract(VectorOneDouble(), Alpha);

	const VectorRegister4Double RotationDot = VectorDot4(A, B);
	const VectorRegister4Double QuatRotationDirMask = VectorCompareGE(RotationDot, Zero);
	const VectorRegister4Double NegativeA = VectorSubtract(Zero, A);
	const VectorRegister4Double BiasTimesA = VectorSelect(QuatRotationDirMask, A, NegativeA);
	const VectorRegister4Double BTimesWeight = VectorMultiply(B, Alpha);
	const VectorRegister4Double UnnormalizedResult = VectorMultiplyAdd(BiasTimesA, OneMinusAlpha, BTimesWeight);

	return UnnormalizedResult;
}

/** 
 * Bi-Linear Quaternion interpolation for quaternions stored in VectorRegisters.
 * Result is NOT normalized.
 */
FORCEINLINE VectorRegister4Float VectorBiLerpQuat(VectorRegister4Float P00, VectorRegister4Float P10, VectorRegister4Float P01, VectorRegister4Float P11, VectorRegister4Float FracX, VectorRegister4Float FracY)
{
	return VectorLerpQuat(
		VectorLerpQuat(P00, P10, FracX),
		VectorLerpQuat(P01, P11, FracX),
		FracY);
}

FORCEINLINE VectorRegister4Double VectorBiLerpQuat(VectorRegister4Double P00, VectorRegister4Double P10, VectorRegister4Double P01, VectorRegister4Double P11, VectorRegister4Double FracX, VectorRegister4Double FracY)
{
	return VectorLerpQuat(
		VectorLerpQuat(P00, P10, FracX),
		VectorLerpQuat(P01, P11, FracX),
		FracY);
}

/** 
 * Inverse quaternion ( -X, -Y, -Z, W) 
 */
FORCEINLINE VectorRegister4Float VectorQuaternionInverse(VectorRegister4Float NormalizedQuat)
{
	return VectorMultiply(GlobalVectorConstants::QINV_SIGN_MASK, NormalizedQuat);
}

FORCEINLINE VectorRegister4Double VectorQuaternionInverse(VectorRegister4Double NormalizedQuat)
{
	return VectorMultiply(GlobalVectorConstants::DOUBLE_QINV_SIGN_MASK, NormalizedQuat);
}


/**
 * Rotate a vector using a unit Quaternion.
 *
 * @param Quat Unit Quaternion to use for rotation.
 * @param VectorW0 Vector to rotate. W component must be zero.
 * @return Vector after rotation by Quat.
 */
FORCEINLINE VectorRegister4Float VectorQuaternionRotateVector(VectorRegister4Float Quat, VectorRegister4Float VectorW0)
{
	// Q * V * Q.Inverse
	//const VectorRegister InverseRotation = VectorQuaternionInverse(Quat);
	//const VectorRegister Temp = VectorQuaternionMultiply2(Quat, VectorW0);
	//const VectorRegister Rotated = VectorQuaternionMultiply2(Temp, InverseRotation);

	// Equivalence of above can be shown to be:
	// http://people.csail.mit.edu/bkph/articles/Quaternions.pdf
	// V' = V + 2w(Q x V) + (2Q x (Q x V))
	// refactor:
	// V' = V + w(2(Q x V)) + (Q x (2(Q x V)))
	// T = 2(Q x V);
	// V' = V + w*(T) + (Q x T)

	const VectorRegister4Float QW = VectorReplicate(Quat, 3);
	VectorRegister4Float T = VectorCross(Quat, VectorW0);
	T = VectorAdd(T, T);
	const VectorRegister4Float VTemp0 = VectorMultiplyAdd(QW, T, VectorW0);
	const VectorRegister4Float VTemp1 = VectorCross(Quat, T);
	const VectorRegister4Float Rotated = VectorAdd(VTemp0, VTemp1);
	return Rotated;
}

FORCEINLINE VectorRegister4Double VectorQuaternionRotateVector(VectorRegister4Double Quat, VectorRegister4Double VectorW0)
{
	const VectorRegister4Double QW = VectorReplicate(Quat, 3);
	VectorRegister4Double T = VectorCross(Quat, VectorW0);
	T = VectorAdd(T, T);
	const VectorRegister4Double VTemp0 = VectorMultiplyAdd(QW, T, VectorW0);
	const VectorRegister4Double VTemp1 = VectorCross(Quat, T);
	const VectorRegister4Double Rotated = VectorAdd(VTemp0, VTemp1);
	return Rotated;
}

/**
 * Rotate a vector using the inverse of a unit Quaternion (rotation in the opposite direction).
 *
 * @param Quat Unit Quaternion to use for rotation.
 * @param VectorW0 Vector to rotate. W component must be zero.
 * @return Vector after rotation by the inverse of Quat.
 */
FORCEINLINE VectorRegister4Float VectorQuaternionInverseRotateVector(VectorRegister4Float Quat, VectorRegister4Float VectorW0)
{
	// Q.Inverse * V * Q
	//const VectorRegister InverseRotation = VectorQuaternionInverse(Quat);
	//const VectorRegister Temp = VectorQuaternionMultiply2(InverseRotation, VectorW0);
	//const VectorRegister Rotated = VectorQuaternionMultiply2(Temp, Quat);

	const VectorRegister4Float QInv = VectorQuaternionInverse(Quat);
	return VectorQuaternionRotateVector(QInv, VectorW0);
}

FORCEINLINE VectorRegister4Double VectorQuaternionInverseRotateVector(VectorRegister4Double Quat, VectorRegister4Double VectorW0)
{
	const VectorRegister4Double QInv = VectorQuaternionInverse(Quat);
	return VectorQuaternionRotateVector(QInv, VectorW0);
}

FORCEINLINE VectorRegister4Double VectorMultiply(VectorRegister4Double Vec1, VectorRegister4Float Vec2)
{
	return VectorMultiply(Vec1, VectorRegister4Double(Vec2));
}

FORCEINLINE VectorRegister4Double VectorMultiplyAdd(VectorRegister4Double Vec1, VectorRegister4Float Vec2, VectorRegister4Double Acc)
{
	return VectorMultiplyAdd(Vec1, VectorRegister4Double(Vec2), Acc);
}

FORCEINLINE VectorRegister4Double VectorMultiplyAdd(VectorRegister4Double Vec1, VectorRegister4Float Vec2, VectorRegister4Float Acc)
{
	return VectorMultiplyAdd(Vec1, VectorRegister4Double(Vec2), VectorRegister4Double(Acc));
}

FORCEINLINE VectorRegister4Double VectorSetDouble(double X, double Y, double Z, double W)
{
	return MakeVectorRegisterDouble(X, Y, Z, W);
}

FORCEINLINE VectorRegister4Double VectorSetDouble1(double D)
{
	return VectorSetFloat1(D);
}

FORCEINLINE void VectorStoreDouble3(VectorRegister4Double Vec, double* Ptr)
{
	VectorStoreFloat3(Vec, Ptr);
}

FORCEINLINE void VectorStoreDouble1(VectorRegister4Double Vec, double* Ptr)
{
	VectorStoreFloat1(Vec, Ptr);
}

/**
 * Creates a vector out of three FLOATs and leaves W undefined.
 *
 * @param X		1st float component
 * @param Y		2nd float component
 * @param Z		3rd float component
 * @return		VectorRegister4Float(X, Y, Z, undefined)
 */
FORCEINLINE VectorRegister4Float VectorSetFloat3(float X, float Y, float Z)
{
	return MakeVectorRegisterFloat(X, Y, Z, 0.0f);
}

FORCEINLINE VectorRegister4Double VectorSetFloat3(double X, double Y, double Z)
{
	return MakeVectorRegisterDouble(X, Y, Z, 0.0);
}

FORCEINLINE VectorRegister4Double VectorSetDouble3(double X, double Y, double Z)
{
	return VectorSetFloat3(X, Y, Z);
}

/**
 * Creates a vector out of four FLOATs.
 *
 * @param X		1st float component
 * @param Y		2nd float component
 * @param Z		3rd float component
 * @param W		4th float component
 * @return		VectorRegister4Float(X, Y, Z, W)
 */
FORCEINLINE VectorRegister4Float VectorSet(float X, float Y, float Z, float W)
{
	return MakeVectorRegisterFloat(X, Y, Z, W);
}

FORCEINLINE VectorRegister4Double VectorSet(double X, double Y, double Z, double W)
{
	return MakeVectorRegisterDouble(X, Y, Z, W);
}

// Overload to resolve ambiguous mixes of ints and floats.
FORCEINLINE VectorRegister4Float VectorSet(uint32 X, uint32 Y, uint32 Z, uint32 W)
{
	return VectorSet(float(X), float(Y), float(Z), float(W));
}

FORCEINLINE VectorRegister4Float VectorFractional(VectorRegister4Float Vec)
{
	return VectorSubtract(Vec, VectorTruncate(Vec));
}

FORCEINLINE VectorRegister4Double VectorFractional(VectorRegister4Double Vec)
{
	return VectorSubtract(Vec, VectorTruncate(Vec));
}


FORCEINLINE void VectorQuaternionMultiply(FQuat4f* RESTRICT Result, const FQuat4f* RESTRICT Quat1, const FQuat4f* RESTRICT Quat2)
{
	const VectorRegister4Float Q1 = VectorLoadAligned(Quat1);
	const VectorRegister4Float Q2 = VectorLoadAligned(Quat2);
	const VectorRegister4Float QResult = VectorQuaternionMultiply2(Q1, Q2);
	VectorStoreAligned(QResult, Result);
}

FORCEINLINE void VectorQuaternionMultiply(FQuat4d* RESTRICT Result, const FQuat4d* RESTRICT Quat1, const FQuat4d* RESTRICT Quat2)
{
	// Warning: don't try to cast FQuat4d to VectorRegister4Double, they may not be of the same alignment.
	// We can still use VectorLoadAligned/VectorStoreAligned, because those only enforce 16-byte alignment and fall back to unaligned otherwise.
	const VectorRegister4Double Q1 = VectorLoadAligned(Quat1);
	const VectorRegister4Double Q2 = VectorLoadAligned(Quat2);
	const VectorRegister4Double QResult = VectorQuaternionMultiply2(Q1, Q2);
	VectorStoreAligned(QResult, Result);
}

/**
 * Counts the number of trailing zeros in the bit representation of the value,
 * counting from least-significant bit to most.
 *
 * @param Value the value to determine the number of leading zeros for
 * @return the number of zeros before the first "on" bit
 */
#if defined(_MSC_VER)
#pragma intrinsic( _BitScanForward )
FORCEINLINE uint32 appCountTrailingZeros(uint32 Value)
{
	if (Value == 0)
	{
		return 32;
	}
	unsigned long BitIndex;	// 0-based, where the LSB is 0 and MSB is 31
	_BitScanForward(&BitIndex, Value);	// Scans from LSB to MSB
	return BitIndex;
}
#else // !defined(_MSC_VER)
FORCEINLINE uint32 appCountTrailingZeros(uint32 Value)
{
	if (Value == 0)
	{
		return 32;
	}
	return __builtin_ffs(Value) - 1;
}
#endif // _MSC_VER
