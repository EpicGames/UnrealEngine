// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly
// IWYU pragma: private

#include "Math/UnrealMathUtility.h"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// constexpr floating point vector constant creation functions that bypass SIMD intrinsic setters
//
// Added new functions instead of constexprifying MakeVectorRegisterXyz to avoid small risk of impacting codegen.
// Long-term, we should only have one set of constexpr make functions without the Constant suffix.
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FORCEINLINE constexpr VectorRegister4Double MakeVectorRegisterDoubleConstant(VectorRegister2Double XY, VectorRegister2Double ZW)
{
	return VectorRegister4Double(XY, ZW, VectorRegisterConstInit{});
}

FORCEINLINE constexpr VectorRegister4Double MakeVectorRegisterDoubleConstant(double X, double Y, double Z, double W)
{
	return MakeVectorRegisterDoubleConstant(MakeVectorRegister2DoubleConstant(X, Y),
											MakeVectorRegister2DoubleConstant(Z, W));
}

FORCEINLINE constexpr VectorRegister4Float MakeVectorRegisterConstant(float X, float Y, float Z, float W)
{
	return MakeVectorRegisterFloatConstant(X, Y, Z, W);
}

FORCEINLINE constexpr VectorRegister4Double MakeVectorRegisterConstant(double X, double Y, double Z, double W)
{
	return MakeVectorRegisterDoubleConstant(X, Y, Z, W);
}

namespace GlobalVectorConstants
{
	inline constexpr VectorRegister4Float FloatOne = MakeVectorRegisterFloatConstant(1.f, 1.f, 1.f, 1.f);
	inline constexpr VectorRegister4Float FloatZero = MakeVectorRegisterFloatConstant(0.f, 0.f, 0.f, 0.f);
	inline constexpr VectorRegister4Float FloatMinusOne = MakeVectorRegisterFloatConstant(-1.f, -1.f, -1.f, -1.f);
	inline constexpr VectorRegister4Float Float0001 = MakeVectorRegisterFloatConstant(0.f, 0.f, 0.f, 1.f);
	inline constexpr VectorRegister4Float Float1000 = MakeVectorRegisterFloatConstant(1.f, 0.f, 0.f, 0.f);
	inline constexpr VectorRegister4Float Float1110 = MakeVectorRegisterFloatConstant(1.f, 1.f, 1.f, 0.f);
	inline constexpr VectorRegister4Float SmallLengthThreshold = MakeVectorRegisterFloatConstant(1.e-8f, 1.e-8f, 1.e-8f, 1.e-8f);
	inline constexpr VectorRegister4Float FloatOneHundredth = MakeVectorRegisterFloatConstant(0.01f, 0.01f, 0.01f, 0.01f);
	inline constexpr VectorRegister4Float Float111_Minus1 = MakeVectorRegisterFloatConstant( 1.f, 1.f, 1.f, -1.f );
	inline constexpr VectorRegister4Float FloatMinus1_111= MakeVectorRegisterFloatConstant( -1.f, 1.f, 1.f, 1.f );
	inline constexpr VectorRegister4Float FloatOneHalf = MakeVectorRegisterFloatConstant( 0.5f, 0.5f, 0.5f, 0.5f );
	inline constexpr VectorRegister4Float FloatMinusOneHalf = MakeVectorRegisterFloatConstant( -0.5f, -0.5f, -0.5f, -0.5f );
	inline constexpr VectorRegister4Float KindaSmallNumber = MakeVectorRegisterFloatConstant( UE_KINDA_SMALL_NUMBER, UE_KINDA_SMALL_NUMBER, UE_KINDA_SMALL_NUMBER, UE_KINDA_SMALL_NUMBER );
	inline constexpr VectorRegister4Float SmallNumber = MakeVectorRegisterFloatConstant( UE_SMALL_NUMBER, UE_SMALL_NUMBER, UE_SMALL_NUMBER, UE_SMALL_NUMBER );
	inline constexpr VectorRegister4Float ThreshQuatNormalized = MakeVectorRegisterFloatConstant( UE_THRESH_QUAT_NORMALIZED, UE_THRESH_QUAT_NORMALIZED, UE_THRESH_QUAT_NORMALIZED, UE_THRESH_QUAT_NORMALIZED );
	inline constexpr VectorRegister4Float BigNumber = MakeVectorRegisterFloatConstant(UE_BIG_NUMBER, UE_BIG_NUMBER, UE_BIG_NUMBER, UE_BIG_NUMBER);

	inline constexpr VectorRegister2Double DoubleOne2d = MakeVectorRegister2DoubleConstant(1.0, 1.0);
	inline constexpr VectorRegister4Double DoubleOne = MakeVectorRegisterDoubleConstant(1.0, 1.0, 1.0, 1.0);
	inline constexpr VectorRegister4Double DoubleZero = MakeVectorRegisterDoubleConstant(0.0, 0.0, 0.0, 0.0);
	inline constexpr VectorRegister4Double DoubleMinusOne = MakeVectorRegisterDoubleConstant(-1.0, -1.0, -1.0, -1.0);
	inline constexpr VectorRegister4Double Double0001 = MakeVectorRegisterDoubleConstant(0.0f, 0.0, 0.0, 1.0);
	inline constexpr VectorRegister4Double Double1000 = MakeVectorRegisterDoubleConstant(1.0, 0.0, 0.0, 0.0);
	inline constexpr VectorRegister4Double Double1110 = MakeVectorRegisterDoubleConstant(1.0, 1.0, 1.0, 0.0);
	inline constexpr VectorRegister4Double DoubleSmallLengthThreshold = MakeVectorRegisterDoubleConstant(1.e-8, 1.e-8, 1.e-8, 1.e-8);
	inline constexpr VectorRegister4Double DoubleOneHundredth = MakeVectorRegisterDoubleConstant(0.01, 0.01, 0.01, 0.01);
	inline constexpr VectorRegister4Double Double111_Minus1 = MakeVectorRegisterDoubleConstant(1., 1., 1., -1.);
	inline constexpr VectorRegister4Double DoubleMinus1_111 = MakeVectorRegisterDoubleConstant(-1., 1., 1., 1.);
	inline constexpr VectorRegister4Double DoubleOneHalf = MakeVectorRegisterDoubleConstant(0.5, 0.5, 0.5, 0.5);
	inline constexpr VectorRegister4Double DoubleMinusOneHalf = MakeVectorRegisterDoubleConstant(-0.5, -0.5, -0.5, -0.5);
	inline constexpr VectorRegister4Double DoubleKindaSmallNumber = MakeVectorRegisterDoubleConstant(UE_DOUBLE_KINDA_SMALL_NUMBER, UE_DOUBLE_KINDA_SMALL_NUMBER, UE_DOUBLE_KINDA_SMALL_NUMBER, UE_DOUBLE_KINDA_SMALL_NUMBER);
	inline constexpr VectorRegister4Double DoubleSmallNumber = MakeVectorRegisterDoubleConstant(UE_DOUBLE_SMALL_NUMBER, UE_DOUBLE_SMALL_NUMBER, UE_DOUBLE_SMALL_NUMBER, UE_DOUBLE_SMALL_NUMBER);
	inline constexpr VectorRegister4Double DoubleThreshQuatNormalized = MakeVectorRegisterDoubleConstant(UE_DOUBLE_THRESH_QUAT_NORMALIZED, UE_DOUBLE_THRESH_QUAT_NORMALIZED, UE_DOUBLE_THRESH_QUAT_NORMALIZED, UE_DOUBLE_THRESH_QUAT_NORMALIZED);
	inline constexpr VectorRegister4Double DoubleBigNumber = MakeVectorRegisterDoubleConstant(UE_DOUBLE_BIG_NUMBER, UE_DOUBLE_BIG_NUMBER, UE_DOUBLE_BIG_NUMBER, UE_DOUBLE_BIG_NUMBER);

	inline constexpr VectorRegister Vector0001 = MakeVectorRegisterConstant(0.0, 0.0, 0.0, 1.0);
	inline constexpr VectorRegister Vector1110 = MakeVectorRegisterConstant(1.0, 1.0, 1.0, 0.0);

	inline constexpr VectorRegister4Int IntOne = MakeVectorRegisterIntConstant(1, 1, 1, 1);
	inline constexpr VectorRegister4Int IntZero = MakeVectorRegisterIntConstant(0, 0, 0, 0);
	inline constexpr VectorRegister4Int IntMinusOne = MakeVectorRegisterIntConstant(-1, -1, -1, -1);

	/** This is to speed up Quaternion Inverse. Static variable to keep sign of inverse **/
	inline constexpr VectorRegister4Float QINV_SIGN_MASK = MakeVectorRegisterFloatConstant( -1.f, -1.f, -1.f, 1.f );
	inline constexpr VectorRegister4Double DOUBLE_QINV_SIGN_MASK = MakeVectorRegisterDoubleConstant(-1., -1., -1., 1.);

	inline constexpr VectorRegister4Float QMULTI_SIGN_MASK0 = MakeVectorRegisterFloatConstant( 1.f, -1.f, 1.f, -1.f );
	inline constexpr VectorRegister4Float QMULTI_SIGN_MASK1 = MakeVectorRegisterFloatConstant( 1.f, 1.f, -1.f, -1.f );
	inline constexpr VectorRegister4Float QMULTI_SIGN_MASK2 = MakeVectorRegisterFloatConstant( -1.f, 1.f, 1.f, -1.f );
	inline constexpr VectorRegister4Double DOUBLE_QMULTI_SIGN_MASK0 = MakeVectorRegisterDoubleConstant(1., -1., 1., -1.);
	inline constexpr VectorRegister4Double DOUBLE_QMULTI_SIGN_MASK1 = MakeVectorRegisterDoubleConstant(1., 1., -1., -1.);
	inline constexpr VectorRegister4Double DOUBLE_QMULTI_SIGN_MASK2 = MakeVectorRegisterDoubleConstant(-1., 1., 1., -1.);

	inline constexpr VectorRegister4Float DEG_TO_RAD = MakeVectorRegisterConstant(UE_PI/(180.f), UE_PI/(180.f), UE_PI/(180.f), UE_PI/(180.f));
	inline constexpr VectorRegister4Float DEG_TO_RAD_HALF = MakeVectorRegisterConstant((UE_PI/180.f)*0.5f, (UE_PI/180.f)*0.5f, (UE_PI/180.f)*0.5f, (UE_PI/180.f)*0.5f);
	inline constexpr VectorRegister4Float RAD_TO_DEG = MakeVectorRegisterConstant((180.f)/UE_PI, (180.f)/UE_PI, (180.f)/UE_PI, (180.f)/UE_PI);
	inline constexpr VectorRegister4Double DOUBLE_DEG_TO_RAD = MakeVectorRegisterConstant(UE_DOUBLE_PI/(180.), UE_DOUBLE_PI/(180.), UE_DOUBLE_PI/(180.), UE_DOUBLE_PI/(180.));
	inline constexpr VectorRegister4Double DOUBLE_DEG_TO_RAD_HALF = MakeVectorRegisterConstant((UE_DOUBLE_PI/180.) * 0.5, (UE_DOUBLE_PI/180.) * 0.5, (UE_DOUBLE_PI/180.) * 0.5, (UE_DOUBLE_PI/180.) * 0.5);
	inline constexpr VectorRegister4Double DOUBLE_RAD_TO_DEG = MakeVectorRegisterConstant((180.)/UE_DOUBLE_PI, (180.)/UE_DOUBLE_PI, (180.)/UE_DOUBLE_PI, (180.)/UE_DOUBLE_PI);

	/** Bitmask to AND out the XYZ components in a vector */
	inline VectorRegister4Float XYZMask()			{ return MakeVectorRegisterFloatMask( ~uint32(0), ~uint32(0), ~uint32(0), uint32(0)); }
	inline VectorRegister4Double DoubleXYZMask()	{ return MakeVectorRegisterDoubleMask(~uint64(0), ~uint64(0), ~uint64(0), uint64(0)); }
	
	/** Bitmask to AND out the sign bit of each components in a vector */
	inline VectorRegister4Float SignBit()			{ return MakeVectorRegisterFloatMask( 0x80000000,  0x80000000,  0x80000000,  0x80000000); }
	inline VectorRegister4Float SignMask()			{ return MakeVectorRegisterFloatMask(~0x80000000, ~0x80000000, ~0x80000000, ~0x80000000); }
	inline constexpr VectorRegister4Int IntSignBit =	MakeVectorRegisterIntConstant( 0x80000000,  0x80000000,  0x80000000,  0x80000000);
	inline constexpr VectorRegister4Int IntSignMask =	MakeVectorRegisterIntConstant(~0x80000000, ~0x80000000, ~0x80000000, ~0x80000000);

	inline constexpr VectorRegister4Int IntAllMask = MakeVectorRegisterIntConstant(			~uint32(0), ~uint32(0), ~uint32(0), ~uint32(0));
	inline VectorRegister4Float AllMask()			{ return MakeVectorRegisterFloatMask(	~uint32(0), ~uint32(0), ~uint32(0), ~uint32(0)); }
	inline VectorRegister4Double DoubleAllMask()	{ return MakeVectorRegisterDoubleMask(	~uint64(0), ~uint64(0), ~uint64(0), ~uint64(0)); }

	inline VectorRegister4Double DoubleSignBit()	{ return MakeVectorRegisterDoubleMask(uint64(1) << 63, uint64(1) << 63, uint64(1) << 63, uint64(1) << 63); }
	inline VectorRegister4Double DoubleSignMask()	{ return MakeVectorRegisterDoubleMask(~(uint64(1) << 63), ~(uint64(1) << 63), ~(uint64(1) << 63), ~(uint64(1) << 63)); }

	/** Vector full of positive infinity */
	inline VectorRegister4Float FloatInfinity()		{ return MakeVectorRegisterFloatMask((uint32)0x7F800000, (uint32)0x7F800000, (uint32)0x7F800000, (uint32)0x7F800000); }
	inline VectorRegister4Double DoubleInfinity()	{ return MakeVectorRegisterDoubleMask((uint64)0x7FF0000000000000, (uint64)0x7FF0000000000000, (uint64)0x7FF0000000000000, (uint64)0x7FF0000000000000); }

	inline constexpr VectorRegister4Float Pi = MakeVectorRegisterConstant(UE_PI, UE_PI, UE_PI, UE_PI);
	inline constexpr VectorRegister4Float TwoPi = MakeVectorRegisterConstant(2.0f*UE_PI, 2.0f*UE_PI, 2.0f*UE_PI, 2.0f*UE_PI);
	inline constexpr VectorRegister4Float PiByTwo = MakeVectorRegisterConstant(0.5f*UE_PI, 0.5f*UE_PI, 0.5f*UE_PI, 0.5f*UE_PI);
	inline constexpr VectorRegister4Float PiByFour = MakeVectorRegisterConstant(0.25f*UE_PI, 0.25f*UE_PI, 0.25f*UE_PI, 0.25f*UE_PI);
	inline constexpr VectorRegister4Float OneOverPi = MakeVectorRegisterConstant(1.0f / UE_PI, 1.0f / UE_PI, 1.0f / UE_PI, 1.0f / UE_PI);
	inline constexpr VectorRegister4Float OneOverTwoPi = MakeVectorRegisterConstant(1.0f / (2.0f*UE_PI), 1.0f / (2.0f*UE_PI), 1.0f / (2.0f*UE_PI), 1.0f / (2.0f*UE_PI));

	inline constexpr VectorRegister4Double DoublePi = MakeVectorRegisterDoubleConstant(UE_DOUBLE_PI, UE_DOUBLE_PI, UE_DOUBLE_PI, UE_DOUBLE_PI);
	inline constexpr VectorRegister4Double DoubleTwoPi = MakeVectorRegisterDoubleConstant(2.0 * UE_DOUBLE_PI, 2.0 * UE_DOUBLE_PI, 2.0 * UE_DOUBLE_PI, 2.0 * UE_DOUBLE_PI);
	inline constexpr VectorRegister4Double DoublePiByTwo = MakeVectorRegisterDoubleConstant(0.5 * UE_DOUBLE_PI, 0.5 * UE_DOUBLE_PI, 0.5 * UE_DOUBLE_PI, 0.5 * UE_DOUBLE_PI);
	inline constexpr VectorRegister4Double DoublePiByFour = MakeVectorRegisterDoubleConstant(0.25 * UE_DOUBLE_PI, 0.25 * UE_DOUBLE_PI, 0.25 * UE_DOUBLE_PI, 0.25 * UE_DOUBLE_PI);
	inline constexpr VectorRegister4Double DoubleOneOverPi = MakeVectorRegisterDoubleConstant(1.0 / UE_DOUBLE_PI, 1.0 / UE_DOUBLE_PI, 1.0 / UE_DOUBLE_PI, 1.0 / UE_DOUBLE_PI);
	inline constexpr VectorRegister4Double DoubleOneOverTwoPi = MakeVectorRegisterDoubleConstant(1.0 / (2.0 * UE_DOUBLE_PI), 1.0 / (2.0 * UE_DOUBLE_PI), 1.0 / (2.0 * UE_DOUBLE_PI), 1.0 / (2.0 * UE_DOUBLE_PI));

	inline constexpr VectorRegister4Float Float255 = MakeVectorRegisterConstant(255.0f, 255.0f, 255.0f, 255.0f);
	inline constexpr VectorRegister4Float Float127 = MakeVectorRegisterConstant(127.0f, 127.0f, 127.0f, 127.0f);
	inline constexpr VectorRegister4Float FloatNeg127 = MakeVectorRegisterConstant(-127.0f, -127.0f, -127.0f, -127.0f);
	inline constexpr VectorRegister4Float Float360 = MakeVectorRegisterConstant(360.f, 360.f, 360.f, 360.f);
	inline constexpr VectorRegister4Float Float180 = MakeVectorRegisterConstant(180.f, 180.f, 180.f, 180.f);

	inline constexpr VectorRegister4Double Double255 = MakeVectorRegisterDoubleConstant(255.0, 255.0, 255.0, 255.0);
	inline constexpr VectorRegister4Double Double127 = MakeVectorRegisterDoubleConstant(127.0, 127.0, 127.0, 127.0);
	inline constexpr VectorRegister4Double DoubleNeg127 = MakeVectorRegisterDoubleConstant(-127.0, -127.0, -127.0, -127.0);
	inline constexpr VectorRegister4Double Double360 = MakeVectorRegisterDoubleConstant(360., 360., 360., 360.);
	inline constexpr VectorRegister4Double Double180 = MakeVectorRegisterDoubleConstant(180., 180., 180., 180.);

	// All float numbers greater than or equal to this have no fractional value.
	inline constexpr VectorRegister4Float FloatNonFractional = MakeVectorRegisterConstant(UE_FLOAT_NON_FRACTIONAL, UE_FLOAT_NON_FRACTIONAL, UE_FLOAT_NON_FRACTIONAL, UE_FLOAT_NON_FRACTIONAL);
	inline constexpr VectorRegister4Double DoubleNonFractional = MakeVectorRegisterDoubleConstant(UE_DOUBLE_NON_FRACTIONAL, UE_DOUBLE_NON_FRACTIONAL, UE_DOUBLE_NON_FRACTIONAL, UE_DOUBLE_NON_FRACTIONAL);

	inline constexpr VectorRegister4Float FloatTwo = MakeVectorRegisterConstant(2.0f, 2.0f, 2.0f, 2.0f);
	inline constexpr uint32 AlmostTwoBits = 0x3fffffff;
	inline VectorRegister4Float FloatAlmostTwo()	{ return MakeVectorRegisterFloatMask(AlmostTwoBits, AlmostTwoBits, AlmostTwoBits, AlmostTwoBits); }

	inline constexpr VectorRegister4Double DoubleTwo = MakeVectorRegisterDoubleConstant(2.0, 2.0, 2.0, 2.0);
	inline constexpr uint64 DoubleAlmostTwoBits = 0x3FFFFFFFFFFFFFFF;
	inline VectorRegister4Double DoubleAlmostTwo()	{ return MakeVectorRegisterDoubleMask(DoubleAlmostTwoBits, DoubleAlmostTwoBits, DoubleAlmostTwoBits, DoubleAlmostTwoBits); }
}
