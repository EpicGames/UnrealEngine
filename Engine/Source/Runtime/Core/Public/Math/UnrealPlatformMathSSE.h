// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Included through other header

#include "CoreTypes.h"
#include "HAL/PlatformMisc.h"
// Code including this header is responsible for including the correct platform-specific header for SSE intrinsics.

#if !PLATFORM_ENABLE_VECTORINTRINSICS || PLATFORM_ENABLE_VECTORINTRINSICS_NEON

template<class Base>
struct TUnrealPlatformMathSSEBase : public Base
{
};

#else

namespace UE4
{
namespace SSE
{
	FORCEINLINE float InvSqrt(float InValue)
	{
		const __m128 One = _mm_set_ss(1.0f);
		const __m128 Y0 = _mm_set_ss(InValue);
		const __m128 X0 = _mm_sqrt_ss(Y0);
		const __m128 R0 = _mm_div_ss(One, X0);
		float temp;
		_mm_store_ss(&temp, R0);
		return temp;
	}

	FORCEINLINE double InvSqrt(double InValue)
	{
		const __m128d One = _mm_set_sd(1.0);
		const __m128d Y0 = _mm_set_sd(InValue);
		const __m128d X0 = _mm_sqrt_sd(One, Y0);
		const __m128d R0 = _mm_div_sd(One, X0);
		double temp;
		_mm_store_sd(&temp, R0);
		return temp;
	}

	FORCEINLINE float InvSqrtEst(float F)
	{
		// Performs one pass of Newton-Raphson iteration on the hardware estimate
		const __m128 fOneHalf = _mm_set_ss(0.5f);
		__m128 Y0, X0, X1, FOver2;
		float temp;

		Y0 = _mm_set_ss(F);
		X0 = _mm_rsqrt_ss(Y0);	// 1/sqrt estimate (12 bits)
		FOver2 = _mm_mul_ss(Y0, fOneHalf);

		// 1st Newton-Raphson iteration
		X1 = _mm_mul_ss(X0, X0);
		X1 = _mm_sub_ss(fOneHalf, _mm_mul_ss(FOver2, X1));
		X1 = _mm_add_ss(X0, _mm_mul_ss(X0, X1));

		_mm_store_ss(&temp, X1);
		return temp;
	}

	FORCEINLINE double InvSqrtEst(double InValue)
	{		
		return InvSqrt(InValue);
	}

	FORCEINLINE int32 TruncToInt32(float F)
	{
		return _mm_cvtt_ss2si(_mm_set_ss(F));
	}

	FORCEINLINE int32 TruncToInt32(double InValue)
	{
		return _mm_cvttsd_si32(_mm_set_sd(InValue));
	}

	FORCEINLINE int64 TruncToInt64(double InValue)
	{
		return _mm_cvttsd_si64(_mm_set_sd(InValue));
	}

	FORCEINLINE int32 FloorToInt32(float F)
	{
		// Note: unlike the Generic solution and the SSE4 float solution, we implement FloorToInt using a rounding instruction, rather than implementing RoundToInt using a floor instruction.  
		// We therefore need to do the same times-2 transform (with a slighly different formula) that RoundToInt does; see the note on RoundToInt
		return _mm_cvt_ss2si(_mm_set_ss(F + F - 0.5f)) >> 1;
	}

	FORCEINLINE int32 FloorToInt32(double InValue)
	{
		return _mm_cvtsd_si32(_mm_set_sd(InValue + InValue - 0.5)) >> 1;
	}

	FORCEINLINE int64 FloorToInt64(double InValue)
	{
		return _mm_cvtsd_si64(_mm_set_sd(InValue + InValue - 0.5)) >> 1;
	}

	FORCEINLINE int32 RoundToInt32(float F)
	{
		// Note: the times-2 is to remove the rounding-to-nearest-even-number behavior that mm_cvt_ss2si uses when the fraction is .5
		// The formula we uses causes the round instruction to always be applied to a an odd integer when the original value was 0.5, and eliminates the rounding-to-nearest-even-number behavior
		// Input -> multiply by two and add .5 -> Round to nearest whole -> divide by two and truncate
		// N -> (2N) + .5 -> 2N (or possibly 2N+1) -> N
		// N + .5 -> 2N + 1.5 -> (round towards even now always means round up) -> 2N + 2 -> N + 1
		return _mm_cvt_ss2si(_mm_set_ss(F + F + 0.5f)) >> 1;
	}

	FORCEINLINE int32 RoundToInt32(double InValue)
	{
		return _mm_cvtsd_si32(_mm_set_sd(InValue + InValue + 0.5)) >> 1;
	}

	FORCEINLINE int64 RoundToInt64(double InValue)
	{
		return _mm_cvtsd_si64(_mm_set_sd(InValue + InValue + 0.5)) >> 1;
	}

	FORCEINLINE int32 CeilToInt32(float F)
	{
		// Note: unlike the Generic solution and the SSE4 float solution, we implement CeilToInt using a rounding instruction, rather than a dedicated ceil instruction
		// We therefore need to do the same times-2 transform (with a slighly different formula) that RoundToInt does; see the note on RoundToInt
		return -(_mm_cvt_ss2si(_mm_set_ss(-0.5f - (F + F))) >> 1);
	}

	FORCEINLINE int32 CeilToInt32(double InValue)
	{
		return -(_mm_cvtsd_si32(_mm_set_sd(-0.5 - (InValue + InValue))) >> 1);
	}

	FORCEINLINE int64 CeilToInt64(double InValue)
	{
		return -(_mm_cvtsd_si64(_mm_set_sd(-0.5 - (InValue + InValue))) >> 1);
	}

	// https://gist.github.com/rygorous/2156668
	// float_to_half_rtne_SSE2
	inline __m128i FloatToHalf(__m128 f)
	{
		const __m128 mask_sign		= _mm_set1_ps(-0.0f);
		const __m128i c_f16max			= _mm_set1_epi32((127 + 16) << 23); // all FP32 values >=this round to +inf
		const __m128i c_nanbit			= _mm_set1_epi32(0x200);
		const __m128i c_nanlobits        = _mm_set1_epi32(0x1ff);
		const __m128i c_infty_as_fp16	= _mm_set1_epi32(0x7c00);
		const __m128i c_min_normal		= _mm_set1_epi32((127 - 14) << 23); // smallest FP32 that yields a normalized FP16
		const __m128i c_subnorm_magic	= _mm_set1_epi32(((127 - 15) + (23 - 10) + 1) << 23);
		const __m128i c_normal_bias		= _mm_set1_epi32(0xfff - ((127 - 15) << 23)); // adjust exponent and add mantissa rounding

		//__m128 justsign	= f & mask_sign;
		__m128 justsign	= _mm_and_ps( f , mask_sign );
		//__m128 absf		= andnot(f, mask_sign); // f & ~mask_sign
		__m128 absf		= _mm_andnot_ps(mask_sign, f); // f & ~mask_sign
		__m128i absf_int		= _mm_castps_si128(absf); // the cast is "free" (extra bypass latency, but no thruput hit)
		__m128 b_isnan	= _mm_cmpunord_ps(absf, absf); // is this a NaN?
		__m128i b_isregular	= _mm_cmpgt_epi32(c_f16max, absf_int); // (sub)normalized or special?
		__m128i nan_payload  = _mm_and_si128(_mm_srli_epi32(absf_int, 13), c_nanlobits); // payload bits for NaNs
		__m128i nan_quiet    = _mm_or_si128(nan_payload, c_nanbit); // and set quiet bit
		__m128i nanfinal		= _mm_and_si128(_mm_castps_si128(b_isnan), nan_quiet);
		__m128i inf_or_nan	= _mm_or_si128(nanfinal, c_infty_as_fp16); // output for specials

		__m128i b_issub		= _mm_cmpgt_epi32(c_min_normal, absf_int);

		// "result is subnormal" path
		__m128 subnorm1	= _mm_add_ps( absf , __m128(_mm_castsi128_ps(c_subnorm_magic)) ); // magic value to round output mantissa
		__m128i subnorm2		= _mm_sub_epi32(_mm_castps_si128(subnorm1), c_subnorm_magic); // subtract out bias

		// "result is normal" path
		__m128i mantoddbit	= _mm_slli_epi32(absf_int, 31 - 13); // shift bit 13 (mantissa LSB) to sign
		__m128i mantodd		= _mm_srai_epi32(mantoddbit, 31); // -1 if FP16 mantissa odd, else 0

		__m128i round1		= _mm_add_epi32(absf_int, c_normal_bias);
		__m128i round2		= _mm_sub_epi32(round1, mantodd); // if mantissa LSB odd, bias towards rounding up (RTNE)
		__m128i normal		= _mm_srli_epi32(round2, 13); // rounded result

		// combine the two non-specials
		__m128i nonspecial	= _mm_or_si128(_mm_and_si128(subnorm2, b_issub), _mm_andnot_si128(b_issub, normal));

		// merge in specials as well
		__m128i joined		= _mm_or_si128(_mm_and_si128(nonspecial, b_isregular), _mm_andnot_si128(b_isregular, inf_or_nan));

		__m128i sign_shift	= _mm_srai_epi32(_mm_castps_si128(justsign), 16);
		__m128i rgba_half32		= _mm_or_si128(joined, sign_shift);

		// there's now a half in each 32-bit lane
		// pack down to 64 bits :
		// packs works because rgba_half32 is sign-extended
        __m128i four_halfs_u64 = _mm_packs_epi32(rgba_half32, rgba_half32);

		return four_halfs_u64;
	}
	
	// four halfs should be in the u64 part of input
	inline __m128 HalfToFloat(__m128i four_halfs_u64)
	{
		__m128i rgba_half32 = _mm_unpacklo_epi16(four_halfs_u64, _mm_setzero_si128());

		const __m128i mask_nosign = _mm_set1_epi32(0x7fff);
		const __m128 magic_mult	= _mm_castsi128_ps(_mm_set1_epi32((254 - 15) << 23));
		const __m128i was_infnan = _mm_set1_epi32(0x7bff);
		const __m128 exp_infnan	= _mm_castsi128_ps(_mm_set1_epi32(255 << 23));
		const __m128i was_nan = _mm_set1_epi32(0x7c00);
		const __m128i nan_quiet	= _mm_set1_epi32(1 << 22);

		__m128i expmant		= _mm_and_si128(mask_nosign, rgba_half32);
		__m128i justsign	= _mm_xor_si128(rgba_half32, expmant);
		__m128i shifted		= _mm_slli_epi32(expmant, 13);
		__m128 scaled		= _mm_mul_ps(_mm_castsi128_ps(shifted), magic_mult);
		__m128i b_wasinfnan = _mm_cmpgt_epi32(expmant, was_infnan);
		__m128i sign	    = _mm_slli_epi32(justsign, 16);
		__m128 infnanexp    = _mm_and_ps(_mm_castsi128_ps(b_wasinfnan), exp_infnan);
		__m128i b_wasnan    = _mm_cmpgt_epi32(expmant, was_nan);
		__m128i nanquiet    = _mm_and_si128(b_wasnan, nan_quiet);
		__m128 infnandone   = _mm_or_ps(infnanexp, _mm_castsi128_ps(nanquiet));

		__m128 sign_inf	= _mm_or_ps(_mm_castsi128_ps(sign), infnandone);
		__m128 result	= _mm_or_ps(scaled, sign_inf);

		return result;
	}
}
}

template<class Base>
struct TUnrealPlatformMathSSEBase : public Base
{
	template<typename T>
	static FORCEINLINE int32 TruncToInt32(T F)
	{
		return UE4::SSE::TruncToInt32(F);
	}

	static FORCEINLINE int64 TruncToInt64(double F)
	{
		return UE4::SSE::TruncToInt64(F);
	}

	template<typename T>
	static FORCEINLINE int32 RoundToInt32(T F)
	{
		return UE4::SSE::RoundToInt32(F);
	}

	static FORCEINLINE int64 RoundToInt64(double F)
	{
		return UE4::SSE::RoundToInt64(F);
	}

	template<typename T>
	static FORCEINLINE int32 FloorToInt32(T F)
	{
		return UE4::SSE::FloorToInt32(F);
	}

	static FORCEINLINE int64 FloorToInt64(double F)
	{
		return UE4::SSE::FloorToInt64(F);
	}

	template<typename T>
	static FORCEINLINE int32 CeilToInt32(T F)
	{
		return UE4::SSE::CeilToInt32(F);
	}

	static FORCEINLINE int64 CeilToInt64(double F)
	{
		return UE4::SSE::CeilToInt64(F);
	}

	//
	// Wrappers for overloads in the base, required since calls declared in base struct won't redirect back to this class
	//

	static FORCEINLINE int32 TruncToInt(float F) { return TruncToInt32(F); }
	static FORCEINLINE int64 TruncToInt(double F) { return TruncToInt64(F); }
	static FORCEINLINE int32 FloorToInt(float F) { return FloorToInt32(F); }
	static FORCEINLINE int64 FloorToInt(double F) { return FloorToInt64(F); }
	static FORCEINLINE int32 RoundToInt(float F) { return RoundToInt32(F); }
	static FORCEINLINE int64 RoundToInt(double F) { return RoundToInt64(F); }
	static FORCEINLINE int32 CeilToInt(float F) { return CeilToInt32(F); }
	static FORCEINLINE int64 CeilToInt(double F) { return CeilToInt64(F); }


	template<typename T>
	static FORCEINLINE T InvSqrt(T F)
	{
		return UE4::SSE::InvSqrt(F);
	}

	template<typename T>
	static FORCEINLINE T InvSqrtEst(T F)
	{
		return UE4::SSE::InvSqrtEst(F);
	}
	
	static FORCEINLINE void VectorStoreHalf(uint16* RESTRICT Dst, const float* RESTRICT Src)
	{
		_mm_storeu_si64((__m128i*)Dst, UE4::SSE::FloatToHalf(_mm_loadu_ps(Src)));
	}
	
	static FORCEINLINE void VectorLoadHalf(float* RESTRICT Dst, const uint16* RESTRICT Src)
	{
		_mm_storeu_ps(Dst, UE4::SSE::HalfToFloat(_mm_loadu_si64((__m128i*)Src)));
	}
};

#endif // PLATFORM_ENABLE_VECTORINTRINSICS