// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Included through other header

#include "HAL/Platform.h"
#include "Math/UnrealPlatformMathSSE.h"

// UE5.2+ requires SSE4.2

// We have to retain this #if because it's pulled in via the linux header chain
// for all platforms at the moment and we rely on the parent class to implement
// the functions
#if PLATFORM_MAYBE_HAS_SSE4_1
#include <smmintrin.h>

namespace UE4
{
namespace SSE4
{
	UE_FORCEINLINE_HINT float TruncToFloat(float F)
	{
		return _mm_cvtss_f32(_mm_round_ps(_mm_set_ss(F), 3));
	}

	UE_FORCEINLINE_HINT double TruncToDouble(double F)
	{
		return _mm_cvtsd_f64(_mm_round_pd(_mm_set_sd(F), 3));
	}

	UE_FORCEINLINE_HINT float FloorToFloat(float F)
	{
		return _mm_cvtss_f32(_mm_floor_ps(_mm_set_ss(F)));
	}

	UE_FORCEINLINE_HINT double FloorToDouble(double F)
	{
		return _mm_cvtsd_f64(_mm_floor_pd(_mm_set_sd(F)));
	}

	UE_FORCEINLINE_HINT float RoundToFloat(float F)
	{
		return FloorToFloat(F + 0.5f);
	}

	UE_FORCEINLINE_HINT double RoundToDouble(double F)
	{
		return FloorToDouble(F + 0.5);
	}

	UE_FORCEINLINE_HINT float CeilToFloat(float F)
	{
		return _mm_cvtss_f32(_mm_ceil_ps(_mm_set_ss(F)));
	}

	UE_FORCEINLINE_HINT double CeilToDouble(double F)
	{
		return _mm_cvtsd_f64(_mm_ceil_pd(_mm_set_sd(F)));
	}
}
}

#endif // PLATFORM_MAYBE_HAS_SSE4_1

#define UNREALPLATFORMMATH_SSE4_1_ENABLED PLATFORM_ALWAYS_HAS_SSE4_1

template<class Base>
struct TUnrealPlatformMathSSE4Base : public TUnrealPlatformMathSSEBase<Base>
{
#if UNREALPLATFORMMATH_SSE4_1_ENABLED

	// Truncate

	static UE_FORCEINLINE_HINT float TruncToFloat(float F)
	{
		return UE4::SSE4::TruncToFloat(F);
	}

	static UE_FORCEINLINE_HINT double TruncToDouble(double F)
	{
		return UE4::SSE4::TruncToDouble(F);
	}

	// Round

	static UE_FORCEINLINE_HINT float RoundToFloat(float F)
	{
		return UE4::SSE4::RoundToFloat(F);
	}

	static UE_FORCEINLINE_HINT double RoundToDouble(double F)
	{
		return UE4::SSE4::RoundToDouble(F);
	}

	// Floor

	static UE_FORCEINLINE_HINT float FloorToFloat(float F)
	{
		return UE4::SSE4::FloorToFloat(F);
	}

	static UE_FORCEINLINE_HINT double FloorToDouble(double F)
	{
		return UE4::SSE4::FloorToDouble(F);
	}

	// Ceil

	static UE_FORCEINLINE_HINT float CeilToFloat(float F)
	{
		return UE4::SSE4::CeilToFloat(F);
	}

	static UE_FORCEINLINE_HINT double CeilToDouble(double F)
	{
		return UE4::SSE4::CeilToDouble(F);
	}


	//
	// Wrappers for overloads in the base, required since calls declared in base struct won't redirect back to this class
	//

	static UE_FORCEINLINE_HINT double TruncToFloat(double F) { return TruncToDouble(F); }
	static UE_FORCEINLINE_HINT double RoundToFloat(double F) { return RoundToDouble(F); }
	static UE_FORCEINLINE_HINT double FloorToFloat(double F) { return FloorToDouble(F); }
	static UE_FORCEINLINE_HINT double CeilToFloat(double F) { return CeilToDouble(F); }

#endif // UNREALPLATFORMMATH_SSE4_ENABLED
};
