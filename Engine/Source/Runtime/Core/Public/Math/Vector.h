// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Math/MathFwd.h" // IWYU pragma: export
#include "Math/NumericLimits.h"
#include "Misc/Crc.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Misc/Parse.h"
#include "Misc/LargeWorldCoordinatesSerializer.h"
#include "Misc/NetworkVersion.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Logging/LogMacros.h"
#include "Math/Vector2D.h"
#include "Misc/ByteSwap.h"
#include "Internationalization/Text.h"
#include "Internationalization/Internationalization.h"
#include "Math/IntVector.h"
#include "Math/Axis.h"
#include "Serialization/MemoryLayout.h"
#include "UObject/ObjectVersion.h"
#include <type_traits>

#if PLATFORM_ENABLE_VECTORINTRINSICS
#include "Math/VectorRegister.h"
#endif

#ifdef _MSC_VER
#pragma warning (push)
// Ensure template functions don't generate shadowing warnings against global variables at the point of instantiation.
#pragma warning (disable : 4459)
#endif

// Move out of global namespace to avoid collisions with Chaos::TVector within the physics code.
namespace UE
{
namespace Math
{

struct TVectorConstInit {};

/**
 * A vector in 3-D space composed of components (X, Y, Z) with floating point precision.
 */
template<typename T>
struct TVector
{
	static_assert(std::is_floating_point_v<T>, "T must be floating point");

public:
	using FReal = T;

	union
	{
		struct
		{
			/** Vector's X component. */
			T X;

			/** Vector's Y component. */
			T Y;

			/** Vector's Z component. */
			T Z;
		};

		UE_DEPRECATED(all, "For internal use only")
		T XYZ[3];
	};

	/** The number of components this vector type has. */
	static constexpr int32 NumComponents = 3;

	/** A zero vector (0,0,0) */
	CORE_API static const TVector<T> ZeroVector;
	
	/** One vector (1,1,1) */
	CORE_API static const TVector<T> OneVector;
	
	/** Unreal up vector (0,0,1) */
	CORE_API static const TVector<T> UpVector;
	
	/** Unreal down vector (0,0,-1) */
	CORE_API static const TVector<T> DownVector;
	
	/** Unreal forward vector (1,0,0) */
	CORE_API static const TVector<T> ForwardVector;
	
	/** Unreal backward vector (-1,0,0) */
	CORE_API static const TVector<T> BackwardVector;
	
	/** Unreal right vector (0,1,0) */
	CORE_API static const TVector<T> RightVector;
	
	/** Unreal left vector (0,-1,0) */
	CORE_API static const TVector<T> LeftVector;
	
	/** Unit X axis vector (1,0,0) */
	CORE_API static const TVector<T> XAxisVector;
	
	/** Unit Y axis vector (0,1,0) */
	CORE_API static const TVector<T> YAxisVector;
	
	/** Unit Z axis vector (0,0,1) */
	CORE_API static const TVector<T> ZAxisVector;

	/** @return Zero Vector (0,0,0) */
	[[nodiscard]] static inline TVector<T> Zero() { return ZeroVector; }

	/** @return One Vector (1,1,1) */
	[[nodiscard]] static inline TVector<T> One() { return OneVector; }

	/** @return Unit X Vector (1,0,0)  */
	[[nodiscard]] static inline TVector<T> UnitX() { return XAxisVector; }

	/** @return Unit Y Vector (0,1,0)  */
	[[nodiscard]] static inline TVector<T> UnitY() { return YAxisVector; }

	/** @return Unit Z Vector (0,0,1)  */
	[[nodiscard]] static inline TVector<T> UnitZ() { return ZAxisVector; }

public:

#if ENABLE_NAN_DIAGNOSTIC
    inline void DiagnosticCheckNaN() const
    {
        if (ContainsNaN())
        {
            logOrEnsureNanError(TEXT("FVector contains NaN: %s"), *ToString());
        }
    }

    inline void DiagnosticCheckNaN(const TCHAR* Message) const
    {
        if (ContainsNaN())
        {
            logOrEnsureNanError(TEXT("%s: FVector contains NaN: %s"), Message, *ToString());
        }
    }
#else
    UE_FORCEINLINE_HINT void DiagnosticCheckNaN() const {}
    UE_FORCEINLINE_HINT void DiagnosticCheckNaN(const TCHAR* Message) const {}
#endif

    /** Default constructor (no initialization). */
    [[nodiscard]] TVector() = default;

    /**
     * Constructor initializing all components to a single T value.
     *
     * @param InF Value to set all components to.
     */
    [[nodiscard]] explicit inline TVector(T InF);

	[[nodiscard]] UE_FORCEINLINE_HINT constexpr TVector(T InF, TVectorConstInit);

    /**
     * Constructor using initial values for each component.
     *
     * @param InX X Coordinate.
     * @param InY Y Coordinate.
     * @param InZ Z Coordinate.
     */
    [[nodiscard]] inline TVector(T InX, T InY, T InZ);

    /**
     * Constructs a vector from an TVector2<T> and Z value.
     * 
     * @param V Vector to copy from.
     * @param InZ Z Coordinate.
     */
    [[nodiscard]] explicit inline TVector(const TVector2<T> V, T InZ);

    /**
     * Constructor using the XYZ components from a 4D vector.
     *
     * @param V 4D Vector to copy from.
     */
    [[nodiscard]] inline TVector(const UE::Math::TVector4<T>& V);

    /**
     * Constructs a vector from an FLinearColor.
     *
     * @param InColor Color to copy from.
     */
    [[nodiscard]] explicit TVector(const FLinearColor& InColor);

    /**
     * Constructs a vector from an FIntVector.
     *
     * @param InVector FIntVector to copy from.
     */
	template <typename IntType>
    [[nodiscard]] explicit TVector(TIntVector3<IntType> InVector);

    /**
     * Constructs a vector from an FIntPoint.
     *
     * @param A Int Point used to set X and Y coordinates, Z is set to zero.
     */
	template <typename IntType>
    [[nodiscard]] explicit TVector(TIntPoint<IntType> A);

    /**
     * Constructor which initializes all components to zero.
     *
     * @param EForceInit Force init enum
     */
    [[nodiscard]] explicit inline TVector(EForceInit);

    /**
     * Calculate cross product between this and another vector.
     *
     * @param V The other vector.
     * @return The cross product.
     */
    [[nodiscard]] inline TVector<T> operator^(const TVector<T>& V) const;

	/**
	 * Calculate cross product between this and another vector.
	 *
	 * @param V The other vector.
	 * @return The cross product.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> Cross(const TVector<T>& V2) const;

    /**
     * Calculate the cross product of two vectors.
     *
     * @param A The first vector.
     * @param B The second vector.
     * @return The cross product.
     */
    [[nodiscard]] UE_FORCEINLINE_HINT static TVector<T> CrossProduct(const TVector<T>& A, const TVector<T>& B);

    /**
     * Calculate the dot product between this and another vector.
     *
     * @param V The other vector.
     * @return The dot product.
     */
    [[nodiscard]] UE_FORCEINLINE_HINT T operator|(const TVector<T>& V) const;

	/**
	 * Calculate the dot product between this and another vector.
	 *
	 * @param V The other vector.
	 * @return The dot product.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T Dot(const TVector<T>& V) const;

    /**
     * Calculate the dot product of two vectors.
     *
     * @param A The first vector.
     * @param B The second vector.
     * @return The dot product.
     */
    [[nodiscard]] UE_FORCEINLINE_HINT static T DotProduct(const TVector<T>& A, const TVector<T>& B);

    /**
     * Gets the result of component-wise addition of this and another vector.
     *
     * @param V The vector to add to this.
     * @return The result of vector addition.
     */
    [[nodiscard]] UE_FORCEINLINE_HINT TVector<T> operator+(const TVector<T>& V) const;

    /**
     * Gets the result of component-wise subtraction of this by another vector.
     *
     * @param V The vector to subtract from this.
     * @return The result of vector subtraction.
     */
    [[nodiscard]] UE_FORCEINLINE_HINT TVector<T> operator-(const TVector<T>& V) const;

    /**
     * Gets the result of subtracting from each component of the vector.
     *
     * @param Bias How much to subtract from each component.
     * @return The result of subtraction.
     */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> operator-(FArg Bias) const
	{
		return TVector<T>(X - (T)Bias, Y - (T)Bias, Z - (T)Bias);
	}

    /**
     * Gets the result of adding to each component of the vector.
     *
     * @param Bias How much to add to each component.
     * @return The result of addition.
     */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> operator+(FArg Bias) const
	{
		return TVector<T>(X + (T)Bias, Y + (T)Bias, Z + (T)Bias);
	}

    /**
     * Gets the result of scaling the vector (multiplying each component by a value).
     *
     * @param Scale What to multiply each component by.
     * @return The result of multiplication.
     */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	[[nodiscard]] UE_FORCEINLINE_HINT TVector<T> operator*(FArg Scale) const
	{
		return TVector<T>(X * (T)Scale, Y * (T)Scale, Z * (T)Scale);
	}

    /**
     * Gets the result of dividing each component of the vector by a value.
     *
     * @param Scale What to divide each component by.
     * @return The result of division.
     */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	[[nodiscard]] TVector<T> operator/(FArg Scale) const
	{
		const T RScale = T(1) / Scale;
		return TVector<T>(X * RScale, Y * RScale, Z * RScale);
	}

    /**
     * Gets the result of component-wise multiplication of this vector by another.
     *
     * @param V The vector to multiply with.
     * @return The result of multiplication.
     */
    [[nodiscard]] UE_FORCEINLINE_HINT TVector<T> operator*(const TVector<T>& V) const;

    /**
     * Gets the result of component-wise division of this vector by another.
     *
     * @param V The vector to divide by.
     * @return The result of division.
     */
    [[nodiscard]] UE_FORCEINLINE_HINT TVector<T> operator/(const TVector<T>& V) const;

    // Binary comparison operators.

    /**
     * Check against another vector for equality.
     *
     * @param V The vector to check against.
     * @return true if the vectors are equal, false otherwise.
     */
   [[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TVector<T>& V) const;

    /**
     * Check against another vector for inequality.
     *
     * @param V The vector to check against.
     * @return true if the vectors are not equal, false otherwise.
     */
    [[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TVector<T>& V) const;

    /**
     * Check against another vector for equality, within specified error limits.
     *
     * @param V The vector to check against.
     * @param Tolerance Error tolerance.
     * @return true if the vectors are equal within tolerance limits, false otherwise.
     */
    [[nodiscard]] UE_FORCEINLINE_HINT bool Equals(const TVector<T>& V, T Tolerance=UE_KINDA_SMALL_NUMBER) const;

    /**
     * Checks whether all components of this vector are the same, within a tolerance.
     *
     * @param Tolerance Error tolerance.
     * @return true if the vectors are equal within tolerance limits, false otherwise.
     */
    [[nodiscard]] UE_FORCEINLINE_HINT bool AllComponentsEqual(T Tolerance=UE_KINDA_SMALL_NUMBER) const;

    /**
     * Get a negated copy of the vector.
     *
     * @return A negated copy of the vector.
     */
    [[nodiscard]] UE_FORCEINLINE_HINT TVector<T> operator-() const;

    /**
     * Adds another vector to this.
     * Uses component-wise addition.
     *
     * @param V Vector to add to this.
     * @return Copy of the vector after addition.
     */
	inline TVector<T> operator+=(const TVector<T>& V);

    /**
     * Subtracts another vector from this.
     * Uses component-wise subtraction.
     *
     * @param V Vector to subtract from this.
     * @return Copy of the vector after subtraction.
     */
    inline TVector<T> operator-=(const TVector<T>& V);

    /**
     * Scales the vector.
     *
     * @param Scale Amount to scale this vector by.
     * @return Copy of the vector after scaling.
     */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	inline TVector<T> operator*=(FArg Scale)
	{
		X *= Scale; Y *= Scale; Z *= Scale;
		DiagnosticCheckNaN();
		return *this;
	}

    /**
     * Divides the vector by a number.
     *
     * @param V What to divide this vector by.
     * @return Copy of the vector after division.
     */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	TVector<T> operator/=(FArg Scale)
	{
		const T RV = (T)1 / Scale;
		X *= RV; Y *= RV; Z *= RV;
		DiagnosticCheckNaN();
		return *this;
	}

    /**
     * Multiplies the vector with another vector, using component-wise multiplication.
     *
     * @param V What to multiply this vector with.
     * @return Copy of the vector after multiplication.
     */
    TVector<T> operator*=(const TVector<T>& V);

    /**
     * Divides the vector by another vector, using component-wise division.
     *
     * @param V What to divide vector by.
     * @return Copy of the vector after division.
     */
    TVector<T> operator/=(const TVector<T>& V);

    /**
     * Gets specific component of the vector.
     *
     * @param Index the index of vector component
     * @return reference to component.
     */
    [[nodiscard]] T& operator[](int32 Index);

    /**
     * Gets specific component of the vector.
     *
     * @param Index the index of vector component
     * @return Copy of the component.
     */
    [[nodiscard]] T operator[](int32 Index)const;

    /**
    * Gets a specific component of the vector.
    *
    * @param Index The index of the component required.
    *
    * @return Reference to the specified component.
    */
    [[nodiscard]] T& Component(int32 Index);

    /**
    * Gets a specific component of the vector.
    *
    * @param Index The index of the component required.
    * @return Copy of the specified component.
    */
    [[nodiscard]] T Component(int32 Index) const;

	/**
	 * Tests if index is valid, i.e. greater than or equal to zero, and less than the number of components in the vector.
	 *
	 * @param Index Index to test.
	 * @returns True if index is valid. False otherwise.
	 */
	bool IsValidIndex(int32 Index) const;

    /** Get a specific component of the vector, given a specific axis by enum */
    [[nodiscard]] T GetComponentForAxis(EAxis::Type Axis) const;

    /** Set a specified componet of the vector, given a specific axis by enum */
    void SetComponentForAxis(EAxis::Type Axis, T Component);

public:

    // Simple functions.

    /**
     * Set the values of the vector directly.
     *
     * @param InX New X coordinate.
     * @param InY New Y coordinate.
     * @param InZ New Z coordinate.
     */
    void Set(T InX, T InY, T InZ);

    /**
     * Get the maximum value of the vector's components.
     *
     * @return The maximum value of the vector's components.
     */
    [[nodiscard]] T GetMax() const;

    /**
     * Get the maximum absolute value of the vector's components.
     *
     * @return The maximum absolute value of the vector's components.
     */
    [[nodiscard]] T GetAbsMax() const;

    /**
     * Get the minimum value of the vector's components.
     *
     * @return The minimum value of the vector's components.
     */
    [[nodiscard]] T GetMin() const;

    /**
     * Get the minimum absolute value of the vector's components.
     *
     * @return The minimum absolute value of the vector's components.
     */
    [[nodiscard]] T GetAbsMin() const;

    /** Gets the component-wise min of two vectors. */
    [[nodiscard]] TVector<T> ComponentMin(const TVector<T>& Other) const;

    /** Gets the component-wise max of two vectors. */
    [[nodiscard]] TVector<T> ComponentMax(const TVector<T>& Other) const;

    /**
     * Get a copy of this vector with absolute value of each component.
     *
     * @return A copy of this vector with absolute value of each component.
     */
    [[nodiscard]] TVector<T> GetAbs() const;

    /**
     * Get the length (magnitude) of this vector.
     *
     * @return The length of this vector.
     */
    [[nodiscard]] T Size() const;

	/**
	 * Get the length (magnitude) of this vector.
	 *
	 * @return The length of this vector.
	 */
	[[nodiscard]] T Length() const;

    /**
     * Get the squared length of this vector.
     *
     * @return The squared length of this vector.
     */
    [[nodiscard]] T SizeSquared() const;

	/**
	 * Get the squared length of this vector.
	 *
	 * @return The squared length of this vector.
	 */
	[[nodiscard]] T SquaredLength() const;

    /**
     * Get the length of the 2D components of this vector.
     *
     * @return The 2D length of this vector.
     */
    [[nodiscard]] T Size2D() const ;

    /**
     * Get the squared length of the 2D components of this vector.
     *
     * @return The squared 2D length of this vector.
     */
    [[nodiscard]] T SizeSquared2D() const ;

    /**
     * Checks whether vector is near to zero within a specified tolerance.
     *
     * @param Tolerance Error tolerance.
     * @return true if the vector is near to zero, false otherwise.
     */
    [[nodiscard]] bool IsNearlyZero(T Tolerance=UE_KINDA_SMALL_NUMBER) const;

    /**
     * Checks whether all components of the vector are exactly zero.
     *
     * @return true if the vector is exactly zero, false otherwise.
     */
    [[nodiscard]] bool IsZero() const;

    /**
     * Check if the vector is of unit length, with specified tolerance.
     *
     * @param LengthSquaredTolerance Tolerance against squared length.
     * @return true if the vector is a unit vector within the specified tolerance.
     */
    [[nodiscard]] UE_FORCEINLINE_HINT bool IsUnit(T LengthSquaredTolerance = UE_KINDA_SMALL_NUMBER) const;

    /**
     * Checks whether vector is normalized.
     *
     * @return true if normalized, false otherwise.
     */
    [[nodiscard]] bool IsNormalized() const;

    /**
     * Normalize this vector in-place if it is larger than a given tolerance. Leaves it unchanged if not.
     *
     * @param Tolerance Minimum squared length of vector for normalization.
     * @return true if the vector was normalized correctly, false otherwise.
     */
    bool Normalize(T Tolerance=UE_SMALL_NUMBER);

    /**
     * Calculates normalized version of vector without checking for zero length.
     *
     * @return Normalized version of vector.
     * @see GetSafeNormal()
     */
    [[nodiscard]] inline TVector<T> GetUnsafeNormal() const;

    /**
     * Gets a normalized copy of the vector, checking it is safe to do so based on the length.
     * Returns zero vector by default if vector length is too small to safely normalize.
     *
     * @param Tolerance Minimum squared vector length. Must be positive.
     * @return A normalized copy if safe, ResultIfZero otherwise.
     */
    [[nodiscard]] TVector<T> GetSafeNormal(T Tolerance=UE_SMALL_NUMBER, const TVector<T>& ResultIfZero = ZeroVector) const;

    /**
     * Gets a normalized copy of the 2D components of the vector, checking it is safe to do so. Z is set to zero. 
     * Returns zero vector by default if vector length is too small to normalize.
     *
     * @param Tolerance Minimum squared vector length. Must be positive.
     * @return Normalized copy if safe, otherwise returns ResultIfZero.
     */
    [[nodiscard]] TVector<T> GetSafeNormal2D(T Tolerance=UE_SMALL_NUMBER, const TVector<T>& ResultIfZero = ZeroVector) const;

    /**
     * Util to convert this vector into a unit direction vector and its original length.
     *
     * @param OutDir Reference passed in to store unit direction vector.
     * @param OutLength Reference passed in to store length of the vector.
     */
	void ToDirectionAndLength(TVector<T>& OutDir, double& OutLength) const;
	void ToDirectionAndLength(TVector<T>& OutDir, float& OutLength) const;

    /**
     * Get a copy of the vector as sign only.
     * Each component is set to +1 or -1, with the sign of zero treated as +1.
     *
     * @param A copy of the vector with each component set to +1 or -1
     */
    [[nodiscard]] inline TVector<T> GetSignVector() const;

    /**
     * Projects 2D components of vector based on Z.
     *
     * @return Projected version of vector based on Z.
     */
    [[nodiscard]] TVector<T> Projection() const;

    /**
    * Calculates normalized 2D version of vector without checking for zero length.
    *
    * @return Normalized version of vector.
    * @see GetSafeNormal2D()
    */
    [[nodiscard]] inline TVector<T> GetUnsafeNormal2D() const;

    /**
     * Gets a copy of this vector snapped to a grid.
     *
     * @param GridSz Grid dimension.
     * @return A copy of this vector snapped to a grid.
     * @see FMath::GridSnap()
     */
    [[nodiscard]] TVector<T> GridSnap(const T& GridSz) const;

    /**
     * Get a copy of this vector, clamped inside of a cube.
     *
     * @param Radius Half size of the cube.
     * @return A copy of this vector, bound by cube.
     */
    [[nodiscard]] TVector<T> BoundToCube(T Radius) const;

    /** Get a copy of this vector, clamped inside of a cube. */
    [[nodiscard]] TVector<T> BoundToBox(const TVector<T>& Min, const TVector<T>& Max) const;

    /** Create a copy of this vector, with its magnitude clamped between Min and Max. */
    [[nodiscard]] TVector<T> GetClampedToSize(T Min, T Max) const;

    /** Create a copy of this vector, with the 2D magnitude clamped between Min and Max. Z is unchanged. */
    [[nodiscard]] TVector<T> GetClampedToSize2D(T Min, T Max) const;

    /** Create a copy of this vector, with its maximum magnitude clamped to MaxSize. */
	[[nodiscard]] TVector<T> GetClampedToMaxSize(T MaxSize) const;

    /** Create a copy of this vector, with the maximum 2D magnitude clamped to MaxSize. Z is unchanged. */
    [[nodiscard]] TVector<T> GetClampedToMaxSize2D(T MaxSize) const;

    /**
     * Add a vector to this and clamp the result in a cube.
     *
     * @param V Vector to add.
     * @param Radius Half size of the cube.
     */
    void AddBounded(const TVector<T>& V, T Radius=MAX_int16);

    /**
     * Gets the reciprocal of this vector, avoiding division by zero.
     * Zero components are set to BIG_NUMBER.
     *
     * @return Reciprocal of this vector.
     */
    [[nodiscard]] TVector<T> Reciprocal() const;

    /**
     * Check whether X, Y and Z are nearly equal.
     *
     * @param Tolerance Specified Tolerance.
     * @return true if X == Y == Z within the specified tolerance.
     */
    [[nodiscard]] bool IsUniform(T Tolerance=UE_KINDA_SMALL_NUMBER) const;

    /**
     * Mirror a vector about a normal vector.
     *
     * @param MirrorNormal Normal vector to mirror about.
     * @return Mirrored vector.
     */
    [[nodiscard]] TVector<T> MirrorByVector(const TVector<T>& MirrorNormal) const;

    /**
     * Mirrors a vector about a plane.
     *
     * @param Plane Plane to mirror about.
     * @return Mirrored vector.
     */
    [[nodiscard]] TVector<T> MirrorByPlane(const TPlane<T>& Plane) const;

    /**
     * Rotates around Axis (assumes Axis.Size() == 1).
     *
     * @param AngleDeg Angle to rotate (in degrees).
     * @param Axis Axis to rotate around.
     * @return Rotated Vector.
     */
    [[nodiscard]] TVector<T> RotateAngleAxis(const T AngleDeg, const TVector<T>& Axis) const;
    
    /**
     * Rotates around Axis (assumes Axis.Size() == 1).
     *
     * @param AngleRad Angle to rotate (in radians).
     * @param Axis Axis to rotate around.
     * @return Rotated Vector.
     */
    [[nodiscard]] TVector<T> RotateAngleAxisRad(const T AngleRad, const TVector<T>& Axis) const;

    /**
     * Returns the cosine of the angle between this vector and another projected onto the XY plane (no Z).
     *
     * @param B the other vector to find the 2D cosine of the angle with.
     * @return The cosine.
     */
    [[nodiscard]] inline T CosineAngle2D(TVector<T> B) const;

    /**
     * Gets a copy of this vector projected onto the input vector.
     *
     * @param A	Vector to project onto, does not assume it is normalized.
     * @return Projected vector.
     */
    [[nodiscard]] UE_FORCEINLINE_HINT TVector<T> ProjectOnTo(const TVector<T>& A) const ;

    /**
     * Gets a copy of this vector projected onto the input vector, which is assumed to be unit length.
     * 
     * @param  Normal Vector to project onto (assumed to be unit length).
     * @return Projected vector.
     */
    [[nodiscard]] UE_FORCEINLINE_HINT TVector<T> ProjectOnToNormal(const TVector<T>& Normal) const;

    /**
     * Return the TRotator orientation corresponding to the direction in which the vector points.
     * Sets Yaw and Pitch to the proper numbers, and sets Roll to zero because the roll can'T be determined from a vector.
     *
     * @return TRotator from the Vector's direction, without any roll.
     * @see ToOrientationQuat()
     */
    [[nodiscard]] CORE_API TRotator<T> ToOrientationRotator() const;

    /**
     * Return the Quaternion orientation corresponding to the direction in which the vector points.
     * Similar to the UE::Math::TRotator<T> version, returns a result without roll such that it preserves the up vector.
     *
     * @note If you don'T care about preserving the up vector and just want the most direct rotation, you can use the faster
     * 'FQuat::FindBetweenVectors(FVector::ForwardVector, YourVector)' or 'FQuat::FindBetweenNormals(...)' if you know the vector is of unit length.
     *
     * @return Quaternion from the Vector's direction, without any roll.
     * @see ToOrientationRotator(), FQuat::FindBetweenVectors()
     */
    [[nodiscard]] CORE_API TQuat<T> ToOrientationQuat() const;

    /**
     * Return the UE::Math::TRotator<T> orientation corresponding to the direction in which the vector points.
     * Sets Yaw and Pitch to the proper numbers, and sets Roll to zero because the roll can't be determined from a vector.
     * @note Identical to 'ToOrientationRotator()' and preserved for legacy reasons.
     * @return UE::Math::TRotator<T> from the Vector's direction.
     * @see ToOrientationRotator(), ToOrientationQuat()
     */
	[[nodiscard]] UE_FORCEINLINE_HINT UE::Math::TRotator<T> Rotation() const
	{
		return ToOrientationRotator();
	}

    /**
     * Find good arbitrary axis vectors to represent U and V axes of a plane,
     * using this vector as the normal of the plane.
     *
     * @param Axis1 Reference to first axis.
     * @param Axis2 Reference to second axis.
     */
    void FindBestAxisVectors(TVector<T>& Axis1, TVector<T>& Axis2) const;

    /** When this vector contains Euler angles (degrees), ensure that angles are between +/-180 */
    void UnwindEuler();

    /**
     * Utility to check if there are any non-finite values (NaN or Inf) in this vector.
     *
     * @return true if there are any non-finite values in this vector, false otherwise.
     */
    bool ContainsNaN() const;

    /**
     * Get a textual representation of this vector.
     *
     * @return A string describing the vector.
     */
    [[nodiscard]] FString ToString() const;

    /**
    * Get a locale aware textual representation of this vector.
    *
    * @return A string describing the vector.
    */
   [[nodiscard]]  FText ToText() const;

    /** Get a short textural representation of this vector, for compact readable logging. */
    [[nodiscard]] FString ToCompactString() const;

    /** Get a short locale aware textural representation of this vector, for compact readable logging. */
    [[nodiscard]] FText ToCompactText() const;

    /**
     * Initialize this Vector based on an FString. The String is expected to contain X=, Y=, Z=.
     * The TVector<T> will be bogus when InitFromString returns false.
     *
     * @param	InSourceString	FString containing the vector values.
     * @return true if the X,Y,Z values were read successfully; false otherwise.
     */
    bool InitFromString(const FString& InSourceString);

	/**
	 * Initialize this Vector based on an FString. The String is expected to contain V(0)
	 * or at least one value X=, Y=, Z=, previously produced by ToCompactString()
	 * The TVector<T> will be bogus when InitFromString returns false.
	 *
	 * @param	InSourceString	FString containing the vector values.
	 * @return true if any of the X,Y,Z values were read successfully; false otherwise.
	 */
	bool InitFromCompactString(const FString& InSourceString);

    /** 
     * Converts a Cartesian unit vector into spherical coordinates on the unit sphere.
     * @return Output Theta will be in the range [0, PI], and output Phi will be in the range [-PI, PI]. 
     */
    [[nodiscard]] TVector2<T> UnitCartesianToSpherical() const;

    /**
     * Convert a direction vector into a 'heading' angle.
     *
     * @return 'Heading' angle between +/-PI. 0 is pointing down +X.
     */
    [[nodiscard]] T HeadingAngle() const;

	/**
	 * Interpolate from a vector to the direction of another vector along a spherical path.
	 * 
	 * @param V Vector we interpolate from
	 * @param Direction Target direction we interpolate to
	 * @param Alpha interpolation amount, usually between 0-1
	 * @return Vector after interpolating between Vector and Direction along a spherical path. The magnitude will remain the length of the starting vector.
	 */
	[[nodiscard]] static CORE_API TVector<T> SlerpVectorToDirection(const TVector<T>& V, const TVector<T>& Direction, T Alpha);

	/**
	 * Interpolate from normalized vector A to normalized vector B along a spherical path.
	 *
	 * @param NormalA Start direction of interpolation, must be normalized.
	 * @param NormalB End target direction of interpolation, must be normalized.
	 * @param Alpha interpolation amount, usually between 0-1
	 * @return Normalized vector after interpolating between NormalA and NormalB along a spherical path.
	 */
	[[nodiscard]] static CORE_API TVector<T> SlerpNormals(const TVector<T>& NormalA, const TVector<T>& NormalB, T Alpha);

    /**
     * Create an orthonormal basis from a basis with at least two orthogonal vectors.
     * It may change the directions of the X and Y axes to make the basis orthogonal,
     * but it won'T change the direction of the Z axis.
     * All axes will be normalized.
     *
     * @param XAxis The input basis' XAxis, and upon return the orthonormal basis' XAxis.
     * @param YAxis The input basis' YAxis, and upon return the orthonormal basis' YAxis.
     * @param ZAxis The input basis' ZAxis, and upon return the orthonormal basis' ZAxis.
     */
    static void CreateOrthonormalBasis(TVector<T>& XAxis,TVector<T>& YAxis,TVector<T>& ZAxis);

    /**
     * Compare two points and see if they're the same, using a threshold.
     *
     * @param P First vector.
     * @param Q Second vector.
     * @return Whether points are the same within a threshold. Uses fast distance approximation (linear per-component distance).
     */
    [[nodiscard]] static bool PointsAreSame(const TVector<T> &P, const TVector<T> &Q);
    
    /**
     * Compare two points and see if they're within specified distance.
     *
     * @param Point1 First vector.
     * @param Point2 Second vector.
     * @param Dist Specified distance.
     * @return Whether two points are within the specified distance. Uses fast distance approximation (linear per-component distance).
     */
    [[nodiscard]] static bool PointsAreNear(const TVector<T> &Point1, const TVector<T> &Point2, T Dist);

    /**
     * Calculate the signed distance (in the direction of the normal) between a point and a plane.
     *
     * @param Point The Point we are checking.
     * @param PlaneBase The Base Point in the plane.
     * @param PlaneNormal The Normal of the plane (assumed to be unit length).
     * @return Signed distance between point and plane.
     */
    [[nodiscard]] static T PointPlaneDist(const TVector<T> &Point, const TVector<T> &PlaneBase, const TVector<T> &PlaneNormal);

    /**
     * Calculate the projection of a point on the given plane.
     *
     * @param Point The point to project onto the plane
     * @param Plane The plane
     * @return Projection of Point onto Plane
     */
	[[nodiscard]] static TVector<T> PointPlaneProject(const TVector<T>& Point, const TPlane<T>& Plane);

    /**
     * Calculate the projection of a point on the plane defined by counter-clockwise (CCW) points A,B,C.
     *
     * @param Point The point to project onto the plane
     * @param A 1st of three points in CCW order defining the plane 
     * @param B 2nd of three points in CCW order defining the plane
     * @param C 3rd of three points in CCW order defining the plane
     * @return Projection of Point onto plane ABC
     */
	[[nodiscard]] static TVector<T> PointPlaneProject(const TVector<T>& Point, const TVector<T>& A, const TVector<T>& B, const TVector<T>& C);

    /**
    * Calculate the projection of a point on the plane defined by PlaneBase and PlaneNormal.
    *
    * @param Point The point to project onto the plane
    * @param PlaneBase Point on the plane
    * @param PlaneNorm Normal of the plane (assumed to be unit length).
    * @return Projection of Point onto plane
    */
    [[nodiscard]] static TVector<T> PointPlaneProject(const TVector<T>& Point, const TVector<T>& PlaneBase, const TVector<T>& PlaneNormal);

    /**
     * Calculate the projection of a vector on the plane defined by PlaneNormal.
     * 
     * @param  V The vector to project onto the plane.
     * @param  PlaneNormal Normal of the plane (assumed to be unit length).
     * @return Projection of V onto plane.
     */
    [[nodiscard]] static TVector<T> VectorPlaneProject(const TVector<T>& V, const TVector<T>& PlaneNormal);

    /**
     * Euclidean distance between two points.
     *
     * @param V1 The first point.
     * @param V2 The second point.
     * @return The distance between two points.
     */
    [[nodiscard]] static UE_FORCEINLINE_HINT T Dist(const TVector<T> &V1, const TVector<T> &V2);
    [[nodiscard]] static UE_FORCEINLINE_HINT T Distance(const TVector<T> &V1, const TVector<T> &V2)
	{
		return Dist(V1, V2);
	}

    /**
    * Euclidean distance between two points in the XY plane (ignoring Z).
    *
    * @param V1 The first point.
    * @param V2 The second point.
    * @return The distance between two points in the XY plane.
    */
    [[nodiscard]] static UE_FORCEINLINE_HINT T DistXY(const TVector<T> &V1, const TVector<T> &V2);
    [[nodiscard]] static UE_FORCEINLINE_HINT T Dist2D(const TVector<T> &V1, const TVector<T> &V2)
	{
		return DistXY(V1, V2);
	}

    /**
     * Squared distance between two points.
     *
     * @param V1 The first point.
     * @param V2 The second point.
     * @return The squared distance between two points.
     */
    [[nodiscard]] static UE_FORCEINLINE_HINT T DistSquared(const TVector<T> &V1, const TVector<T> &V2);

    /**
     * Squared distance between two points in the XY plane only.
     *	
     * @param V1 The first point.
     * @param V2 The second point.
     * @return The squared distance between two points in the XY plane
     */
    [[nodiscard]] static UE_FORCEINLINE_HINT T DistSquaredXY(const TVector<T> &V1, const TVector<T> &V2);
    [[nodiscard]] static UE_FORCEINLINE_HINT T DistSquared2D(const TVector<T> &V1, const TVector<T> &V2)
	{
		return DistSquaredXY(V1, V2);
	}
    
    /**
     * Compute pushout of a box from a plane.
     *
     * @param Normal The plane normal.
     * @param Size The size of the box.
     * @return Pushout required.
     */
    [[nodiscard]] static UE_FORCEINLINE_HINT T BoxPushOut(const TVector<T>& Normal, const TVector<T>& Size);

    /**
     * Min, Max, Min3, Max3 like FMath
     */
    [[nodiscard]] static inline TVector<T> Min( const TVector<T>& A, const TVector<T>& B );
    [[nodiscard]] static inline TVector<T> Max( const TVector<T>& A, const TVector<T>& B );

    [[nodiscard]] static inline TVector<T> Min3( const TVector<T>& A, const TVector<T>& B, const TVector<T>& C );
    [[nodiscard]] static inline TVector<T> Max3( const TVector<T>& A, const TVector<T>& B, const TVector<T>& C );

    /**
     * See if two normal vectors are nearly parallel, meaning the angle between them is close to 0 degrees.
     *
     * @param  Normal1 First normalized vector.
     * @param  Normal1 Second normalized vector.
     * @param  ParallelCosineThreshold Normals are parallel if absolute value of dot product (cosine of angle between them) is greater than or equal to this. For example: cos(1.0 degrees).
     * @return true if vectors are nearly parallel, false otherwise.
     */
    [[nodiscard]] static bool Parallel(const TVector<T>& Normal1, const TVector<T>& Normal2, T ParallelCosineThreshold = UE_THRESH_NORMALS_ARE_PARALLEL);

    /**
     * See if two normal vectors are coincident (nearly parallel and point in the same direction).
     * 
     * @param  Normal1 First normalized vector.
     * @param  Normal2 Second normalized vector.
     * @param  ParallelCosineThreshold Normals are coincident if dot product (cosine of angle between them) is greater than or equal to this. For example: cos(1.0 degrees).
     * @return true if vectors are coincident (nearly parallel and point in the same direction), false otherwise.
     */
    [[nodiscard]] static bool Coincident(const TVector<T>& Normal1, const TVector<T>& Normal2, T ParallelCosineThreshold = UE_THRESH_NORMALS_ARE_PARALLEL);

    /**
     * See if two normal vectors are nearly orthogonal (perpendicular), meaning the angle between them is close to 90 degrees.
     * 
     * @param  Normal1 First normalized vector.
     * @param  Normal2 Second normalized vector.
     * @param  OrthogonalCosineThreshold Normals are orthogonal if absolute value of dot product (cosine of angle between them) is less than or equal to this. For example: cos(89.0 degrees).
     * @return true if vectors are orthogonal (perpendicular), false otherwise.
     */
    [[nodiscard]] static bool Orthogonal(const TVector<T>& Normal1, const TVector<T>& Normal2, T OrthogonalCosineThreshold = UE_THRESH_NORMALS_ARE_ORTHOGONAL);

    /**
     * See if two planes are coplanar. They are coplanar if the normals are nearly parallel and the planes include the same set of points.
     *
     * @param Base1 The base point in the first plane.
     * @param Normal1 The normal of the first plane.
     * @param Base2 The base point in the second plane.
     * @param Normal2 The normal of the second plane.
     * @param ParallelCosineThreshold Normals are parallel if absolute value of dot product is greater than or equal to this.
     * @return true if the planes are coplanar, false otherwise.
     */
    [[nodiscard]] static bool Coplanar(const TVector<T>& Base1, const TVector<T>& Normal1, const TVector<T>& Base2, const TVector<T>& Normal2, T ParallelCosineThreshold = UE_THRESH_NORMALS_ARE_PARALLEL);

    /**
     * Triple product of three vectors: X dot (Y cross Z).
     *
     * @param X The first vector.
     * @param Y The second vector.
     * @param Z The third vector.
     * @return The triple product: X dot (Y cross Z).
     */
    [[nodiscard]] static T Triple(const TVector<T>& X, const TVector<T>& Y, const TVector<T>& Z);

    /**
     * Generates a list of sample points on a Bezier curve defined by 2 points.
     *
     * @param ControlPoints	Array of 4 FVectors (vert1, controlpoint1, controlpoint2, vert2).
     * @param NumPoints Number of samples.
     * @param OutPoints Receives the output samples.
     * @return The path length.
     */
    static T EvaluateBezier(const TVector<T>* ControlPoints, int32 NumPoints, TArray<TVector<T>>& OutPoints);

    /**
     * Converts a vector containing radian values to a vector containing degree values.
     *
     * @param RadVector	Vector containing radian values
     * @return Vector  containing degree values
     */
    [[nodiscard]] static TVector<T> RadiansToDegrees(const TVector<T>& RadVector);

    /**
     * Converts a vector containing degree values to a vector containing radian values.
     *
     * @param DegVector	Vector containing degree values
     * @return Vector containing radian values
     */
    [[nodiscard]] static TVector<T> DegreesToRadians(const TVector<T>& DegVector);

    /**
     * Given a current set of cluster centers, a set of points, iterate N times to move clusters to be central. 
     *
     * @param Clusters Reference to array of Clusters.
     * @param Points Set of points.
     * @param NumIterations Number of iterations.
     * @param NumConnectionsToBeValid Sometimes you will have long strings that come off the mass of points
     * which happen to have been chosen as Cluster starting points.  You want to be able to disregard those.
     */
    static void GenerateClusterCenters(TArray<TVector<T>>& Clusters, const TArray<TVector<T>>& Points, int32 NumIterations, int32 NumConnectionsToBeValid);
	
	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}
	
    bool Serialize(FStructuredArchive::FSlot Slot)
    {
        Slot << *this;
        return true;
    }

	bool SerializeFromMismatchedTag(FName StructTag, FStructuredArchive::FSlot Slot)
	{
		if constexpr (std::is_same_v<T, float>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Slot, Vector, Vector3f, Vector3d);
		}
		else
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Slot, Vector, Vector3d, Vector3f);
		}
	}
	
    /** 
     * Network serialization function.
     * FVectors NetSerialize without quantization (ie exact values are serialized). se the FVectors_NetQuantize etc (NetSerialization.h) instead.
     *
     * @see FVector_NetQuantize, FVector_NetQuantize10, FVector_NetQuantize100, FVector_NetQuantizeNormal
     */
	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
	{
		Ar.UsingCustomVersion(FEngineNetworkCustomVersion::Guid);

		if (Ar.EngineNetVer() >= FEngineNetworkCustomVersion::SerializeDoubleVectorsAsDoubles && Ar.EngineNetVer() != FEngineNetworkCustomVersion::Ver21AndViewPitchOnly_DONOTUSE)
		{
			Ar << X << Y << Z;
		}
		else
		{
			checkf(Ar.IsLoading(), TEXT("float -> double conversion applied outside of load!"));
			// Always serialize as float
			float SX, SY, SZ;
			Ar << SX << SY << SZ;
			X = SX;
			Y = SY;
			Z = SZ;
		}
		return true;
	}

	// Conversion from other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TVector(const TVector<FArg>& From) : TVector<T>((T)From.X, (T)From.Y, (T)From.Z) {}
};

/**
 * Serializer for FVector3f.
 *
 * @param Ar Serialization Archive.
 * @param V Vector to serialize.
 */
inline FArchive& operator<<(FArchive& Ar, TVector<float>& V)
{
	// @warning BulkSerialize: FVector3f is serialized as memory dump
	// See TArray::BulkSerialize for detailed description of implied limitations.
	Ar << V.X << V.Y << V.Z;

	V.DiagnosticCheckNaN();

	return Ar;
}

/**
 * Serializer for FVector3d.
 *
 * @param Ar Serialization Archive.
 * @param V Vector to serialize.
 */
inline FArchive& operator<<(FArchive& Ar, TVector<double>& V)
{
	// @warning BulkSerialize: FVector3d is serialized as memory dump
	// See TArray::BulkSerialize for detailed description of implied limitations.
	if (Ar.UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
	{
		Ar << V.X;
		Ar << V.Y;
		Ar << V.Z;
	}
	else
	{
		checkf(Ar.IsLoading(), TEXT("float -> double conversion applied outside of load!"));
		// Stored as floats, so serialize float and copy.
		float X, Y, Z;

		Ar << X;
		Ar << Y;
		Ar << Z;

		V = TVector<double>(X, Y, Z);
	};

	V.DiagnosticCheckNaN();

	return Ar;
}

/**
 * Structured archive slot serializer for FVector3f.
 *
 * @param Slot Structured archive slot.
 * @param V Vector to serialize.
 */

inline void operator<<(FStructuredArchive::FSlot Slot, TVector<float>& V)
{
	// @warning BulkSerialize: FVector3f is serialized as memory dump
	// See TArray::BulkSerialize for detailed description of implied limitations.
	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	Record << SA_VALUE(TEXT("X"), V.X);
	Record << SA_VALUE(TEXT("Y"), V.Y);
	Record << SA_VALUE(TEXT("Z"), V.Z);
	V.DiagnosticCheckNaN();
}

/**
 * Structured archive slot serializer for FVector3d.
 *
 * @param Slot Structured archive slot.
 * @param V Vector to serialize.
 */

inline void operator<<(FStructuredArchive::FSlot Slot, TVector<double>& V)
{
	// @warning BulkSerialize: FVector3d is serialized as memory dump
	// See TArray::BulkSerialize for detailed description of implied limitations.
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	if (Slot.GetUnderlyingArchive().UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
	{
		Record << SA_VALUE(TEXT("X"), V.X);
		Record << SA_VALUE(TEXT("Y"), V.Y);
		Record << SA_VALUE(TEXT("Z"), V.Z);
	}
	else
	{
		checkf(Slot.GetUnderlyingArchive().IsLoading(), TEXT("float -> double conversion applied outside of load!"));
		// Stored as floats, so serialize float and copy.
		float X, Y, Z;
		Record << SA_VALUE(TEXT("X"), X);
		Record << SA_VALUE(TEXT("Y"), Y);
		Record << SA_VALUE(TEXT("Z"), Z);
		V = TVector<double>(X, Y, Z);
	}
	V.DiagnosticCheckNaN();
}

/* FVector inline functions
 *****************************************************************************/

template<typename T>
inline TVector<T>::TVector(const TVector2<T> V, T InZ)
    : X(V.X), Y(V.Y), Z(InZ)
{
    DiagnosticCheckNaN();
}



template<typename T>
inline TVector<T> TVector<T>::RotateAngleAxis(const T AngleDeg, const TVector<T>& Axis) const
{
	return RotateAngleAxisRad(FMath::DegreesToRadians(AngleDeg), Axis);
}

template<typename T>
inline TVector<T> TVector<T>::RotateAngleAxisRad(const T AngleRad, const TVector<T>& Axis) const
{
    T S, C;
    FMath::SinCos(&S, &C, AngleRad);

    const T XX	= Axis.X * Axis.X;
    const T YY	= Axis.Y * Axis.Y;
    const T ZZ	= Axis.Z * Axis.Z;

    const T XY	= Axis.X * Axis.Y;
    const T YZ	= Axis.Y * Axis.Z;
    const T ZX	= Axis.Z * Axis.X;

    const T XS	= Axis.X * S;
    const T YS	= Axis.Y * S;
    const T ZS	= Axis.Z * S;

    const T OMC	= 1.f - C;

    return TVector<T>(
        (OMC * XX + C) * X + (OMC * XY - ZS) * Y + (OMC * ZX + YS) * Z,
        (OMC * XY + ZS) * X + (OMC * YY + C) * Y + (OMC * YZ - XS) * Z,
        (OMC * ZX - YS) * X + (OMC * YZ + XS) * Y + (OMC * ZZ + C) * Z
        );
}

template<typename T>
inline bool TVector<T>::PointsAreSame(const TVector<T>& P, const TVector<T>& Q)
{
    T Temp;
    Temp=P.X-Q.X;
    if((Temp > -UE_THRESH_POINTS_ARE_SAME) && (Temp < UE_THRESH_POINTS_ARE_SAME))
    {
        Temp=P.Y-Q.Y;
        if((Temp > -UE_THRESH_POINTS_ARE_SAME) && (Temp < UE_THRESH_POINTS_ARE_SAME))
        {
            Temp=P.Z-Q.Z;
            if((Temp > -UE_THRESH_POINTS_ARE_SAME) && (Temp < UE_THRESH_POINTS_ARE_SAME))
            {
                return true;
            }
        }
    }
    return false;
}

template<typename T>
inline bool TVector<T>::PointsAreNear(const TVector<T>& Point1, const TVector<T>& Point2, T Dist)
{
    T Temp;
    Temp=(Point1.X - Point2.X); if (FMath::Abs(Temp)>=Dist) return false;
    Temp=(Point1.Y - Point2.Y); if (FMath::Abs(Temp)>=Dist) return false;
    Temp=(Point1.Z - Point2.Z); if (FMath::Abs(Temp)>=Dist) return false;
    return true;
}

template<typename T>
inline T TVector<T>::PointPlaneDist
(
    const TVector<T> &Point,
    const TVector<T> &PlaneBase,
    const TVector<T> &PlaneNormal
)
{
    return (Point - PlaneBase) | PlaneNormal;
}


template<typename T>
inline TVector<T> TVector<T>::PointPlaneProject(const TVector<T>& Point, const TVector<T>& PlaneBase, const TVector<T>& PlaneNorm)
{
    //Find the distance of X from the plane
    //Add the distance back along the normal from the point
    return Point - TVector<T>::PointPlaneDist(Point,PlaneBase,PlaneNorm) * PlaneNorm;
}

template<typename T>
inline TVector<T> TVector<T>::VectorPlaneProject(const TVector<T>& V, const TVector<T>& PlaneNormal)
{
    return V - V.ProjectOnToNormal(PlaneNormal);
}

template<typename T>
inline bool TVector<T>::Parallel(const TVector<T>& Normal1, const TVector<T>& Normal2, T ParallelCosineThreshold)
{
    const T NormalDot = Normal1 | Normal2;
    return FMath::Abs(NormalDot) >= ParallelCosineThreshold;
}

template<typename T>
inline bool TVector<T>::Coincident(const TVector<T>& Normal1, const TVector<T>& Normal2, T ParallelCosineThreshold)
{
    const T NormalDot = Normal1 | Normal2;
    return NormalDot >= ParallelCosineThreshold;
}

template<typename T>
inline bool TVector<T>::Orthogonal(const TVector<T>& Normal1, const TVector<T>& Normal2, T OrthogonalCosineThreshold)
{
    const T NormalDot = Normal1 | Normal2;
    return FMath::Abs(NormalDot) <= OrthogonalCosineThreshold;
}

template<typename T>
inline bool TVector<T>::Coplanar(const TVector<T>& Base1, const TVector<T>& Normal1, const TVector<T>& Base2, const TVector<T>& Normal2, T ParallelCosineThreshold)
{
    if      (!TVector<T>::Parallel(Normal1,Normal2,ParallelCosineThreshold)) return false;
    else if (FMath::Abs(TVector<T>::PointPlaneDist (Base2,Base1,Normal1)) > UE_THRESH_POINT_ON_PLANE) return false;
    else return true;
}

template<typename T>
inline T TVector<T>::Triple(const TVector<T>& X, const TVector<T>& Y, const TVector<T>& Z)
{
    return
    (	(X.X * (Y.Y * Z.Z - Y.Z * Z.Y))
    +	(X.Y * (Y.Z * Z.X - Y.X * Z.Z))
    +	(X.Z * (Y.X * Z.Y - Y.Y * Z.X)));
}

template<typename T>
inline TVector<T> TVector<T>::RadiansToDegrees(const TVector<T>& RadVector)
{
    return RadVector * (180.f / UE_PI);
}

template<typename T>
inline TVector<T> TVector<T>::DegreesToRadians(const TVector<T>& DegVector)
{
    return DegVector * (UE_PI / 180.f);
}

template<typename T>
inline TVector<T>::TVector(T InF)
    : X(InF), Y(InF), Z(InF)
{
    DiagnosticCheckNaN();
}

template<typename T>
UE_FORCEINLINE_HINT constexpr TVector<T>::TVector(T InF, TVectorConstInit)
	: X(InF), Y(InF), Z(InF)
{
}

template<typename T>
inline TVector<T>::TVector(T InX, T InY, T InZ)
    : X(InX), Y(InY), Z(InZ)
{
    DiagnosticCheckNaN();
}

template<typename T>
inline TVector<T>::TVector(const FLinearColor& InColor)
    : X(InColor.R), Y(InColor.G), Z(InColor.B)
{
    DiagnosticCheckNaN();
}

template<typename T>
template <typename IntType>
inline TVector<T>::TVector(TIntVector3<IntType> InVector)
    : X((T)InVector.X), Y((T)InVector.Y), Z((T)InVector.Z)
{
    DiagnosticCheckNaN();
}

template<typename T>
template <typename IntType>
inline TVector<T>::TVector(TIntPoint<IntType> A)
    : X((T)A.X), Y((T)A.Y), Z(0.f)
{
    DiagnosticCheckNaN();
}

template<typename T>
inline TVector<T>::TVector(EForceInit)
    : X(0.0f), Y(0.0f), Z(0.0f)
{
    DiagnosticCheckNaN();
}

template<typename T>
inline TVector<T> TVector<T>::operator^(const TVector<T>& V) const
{
    return TVector<T>
        (
        Y * V.Z - Z * V.Y,
        Z * V.X - X * V.Z,
        X * V.Y - Y * V.X
        );
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TVector<T>::Cross(const TVector<T>& V) const
{
	return *this ^ V;
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TVector<T>::CrossProduct(const TVector<T>& A, const TVector<T>& B)
{
    return A ^ B;
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::operator|(const TVector<T>& V) const
{
    return X*V.X + Y*V.Y + Z*V.Z;
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::Dot(const TVector<T>& V) const
{
	return *this | V;
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::DotProduct(const TVector<T>& A, const TVector<T>& B)
{
    return A | B;
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TVector<T>::operator+(const TVector<T>& V) const
{
    return TVector<T>(X + V.X, Y + V.Y, Z + V.Z);
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TVector<T>::operator-(const TVector<T>& V) const
{
    return TVector<T>(X - V.X, Y - V.Y, Z - V.Z);
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TVector<T>::operator*(const TVector<T>& V) const
{
    return TVector<T>(X * V.X, Y * V.Y, Z * V.Z);
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TVector<T>::operator/(const TVector<T>& V) const
{
    return TVector<T>(X / V.X, Y / V.Y, Z / V.Z);
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector<T>::operator==(const TVector<T>& V) const
{
    return X==V.X && Y==V.Y && Z==V.Z;
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector<T>::operator!=(const TVector<T>& V) const
{
    return X!=V.X || Y!=V.Y || Z!=V.Z;
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector<T>::Equals(const TVector<T>& V, T Tolerance) const
{
    return FMath::Abs(X-V.X) <= Tolerance && FMath::Abs(Y-V.Y) <= Tolerance && FMath::Abs(Z-V.Z) <= Tolerance;
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector<T>::AllComponentsEqual(T Tolerance) const
{
    return FMath::Abs(X - Y) <= Tolerance && FMath::Abs(X - Z) <= Tolerance && FMath::Abs(Y - Z) <= Tolerance;
}


template<typename T>
UE_FORCEINLINE_HINT TVector<T> TVector<T>::operator-() const
{
    return TVector<T>(-X, -Y, -Z);
}


template<typename T>
inline TVector<T> TVector<T>::operator+=(const TVector<T>& V)
{
    X += V.X; Y += V.Y; Z += V.Z;
    DiagnosticCheckNaN();
    return *this;
}

template<typename T>
inline TVector<T> TVector<T>::operator-=(const TVector<T>& V)
{
    X -= V.X; Y -= V.Y; Z -= V.Z;
    DiagnosticCheckNaN();
    return *this;
}

template<typename T>
inline TVector<T> TVector<T>::operator*=(const TVector<T>& V)
{
    X *= V.X; Y *= V.Y; Z *= V.Z;
    DiagnosticCheckNaN();
    return *this;
}

template<typename T>
inline TVector<T> TVector<T>::operator/=(const TVector<T>& V)
{
    X /= V.X; Y /= V.Y; Z /= V.Z;
    DiagnosticCheckNaN();
    return *this;
}

template<typename T>
UE_FORCEINLINE_HINT T& TVector<T>::operator[](int32 Index)
{
	return Component(Index);
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::operator[](int32 Index)const
{
	return Component(Index);
}

template<typename T>
inline void TVector<T>::Set(T InX, T InY, T InZ)
{
    X = InX;
    Y = InY;
    Z = InZ;
    DiagnosticCheckNaN();
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::GetMax() const
{
    return FMath::Max(FMath::Max(X,Y),Z);
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::GetAbsMax() const
{
    return FMath::Max(FMath::Max(FMath::Abs(X),FMath::Abs(Y)),FMath::Abs(Z));
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::GetMin() const
{
    return FMath::Min(FMath::Min(X,Y),Z);
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::GetAbsMin() const
{
    return FMath::Min(FMath::Min(FMath::Abs(X),FMath::Abs(Y)),FMath::Abs(Z));
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TVector<T>::ComponentMin(const TVector<T>& Other) const
{
    return TVector<T>(FMath::Min(X, Other.X), FMath::Min(Y, Other.Y), FMath::Min(Z, Other.Z));
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TVector<T>::ComponentMax(const TVector<T>& Other) const
{
    return TVector<T>(FMath::Max(X, Other.X), FMath::Max(Y, Other.Y), FMath::Max(Z, Other.Z));
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TVector<T>::GetAbs() const
{
    return TVector<T>(FMath::Abs(X), FMath::Abs(Y), FMath::Abs(Z));
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::Size() const
{
    return FMath::Sqrt(X*X + Y*Y + Z*Z);
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::Length() const
{
	return Size();
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::SizeSquared() const
{
    return X*X + Y*Y + Z*Z;
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::SquaredLength() const
{
	return SizeSquared();
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::Size2D() const
{
    return FMath::Sqrt(X*X + Y*Y);
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::SizeSquared2D() const
{
    return X*X + Y*Y;
}

template<typename T>
inline bool TVector<T>::IsNearlyZero(T Tolerance) const
{
    return
        FMath::Abs(X)<=Tolerance
        &&	FMath::Abs(Y)<=Tolerance
        &&	FMath::Abs(Z)<=Tolerance;
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector<T>::IsZero() const
{
    return X==0.f && Y==0.f && Z==0.f;
}

template<typename T>
inline bool TVector<T>::Normalize(T Tolerance)
{
    const T SquareSum = X*X + Y*Y + Z*Z;
    if(SquareSum > Tolerance)
    {
        const T Scale = FMath::InvSqrt(SquareSum);
        X *= Scale; Y *= Scale; Z *= Scale;
        return true;
    }
    return false;
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector<T>::IsUnit(T LengthSquaredTolerance) const
{
    return FMath::Abs(1.0f - SizeSquared()) < LengthSquaredTolerance;
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector<T>::IsNormalized() const
{
    return (FMath::Abs(1.f - SizeSquared()) < UE_THRESH_VECTOR_NORMALIZED);
}

template<typename T>
inline void TVector<T>::ToDirectionAndLength(TVector<T>& OutDir, double& OutLength) const
{
	OutLength = Size();
	if (OutLength > UE_SMALL_NUMBER)
	{
		T OneOverLength = T(1.0 / OutLength);
		OutDir = TVector<T>(X * OneOverLength, Y * OneOverLength, Z * OneOverLength);
	}
	else
	{
		OutDir = ZeroVector;
	}
}

template<typename T>
inline void TVector<T>::ToDirectionAndLength(TVector<T>& OutDir, float& OutLength) const
{
	OutLength = (float)Size();
	if (OutLength > UE_SMALL_NUMBER)
	{
		float OneOverLength = 1.0f / OutLength;
		OutDir = TVector<T>(X * OneOverLength, Y * OneOverLength, Z * OneOverLength);
	}
	else
	{
		OutDir = ZeroVector;
	}
}


template<typename T>
inline TVector<T> TVector<T>::GetSignVector() const
{
    return TVector<T>
        (
        FMath::FloatSelect(X, (T)1, (T)-1),		// LWC_TODO: Templatize FMath functionality
        FMath::FloatSelect(Y, (T)1, (T)-1),
        FMath::FloatSelect(Z, (T)1, (T)-1)
        );
}

template<typename T>
inline TVector<T> TVector<T>::Projection() const
{
    const T RZ = 1.f/Z;
    return TVector<T>(X*RZ, Y*RZ, 1);
}

template<typename T>
inline TVector<T> TVector<T>::GetUnsafeNormal() const
{
    const T Scale = FMath::InvSqrt(X*X+Y*Y+Z*Z);
    return TVector<T>(X*Scale, Y*Scale, Z*Scale);
}

template<typename T>
inline TVector<T> TVector<T>::GetUnsafeNormal2D() const
{
    const T Scale = FMath::InvSqrt(X * X + Y * Y);
    return TVector<T>(X*Scale, Y*Scale, 0);
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TVector<T>::GridSnap(const T& GridSz) const
{
    return TVector<T>(FMath::GridSnap(X, GridSz),FMath::GridSnap(Y, GridSz),FMath::GridSnap(Z, GridSz));
}

template<typename T>
inline TVector<T> TVector<T>::BoundToCube(T Radius) const
{
    return TVector<T>
        (
        FMath::Clamp(X,-Radius,Radius),
        FMath::Clamp(Y,-Radius,Radius),
        FMath::Clamp(Z,-Radius,Radius)
        );
}

template<typename T>
inline TVector<T> TVector<T>::BoundToBox(const TVector<T>& Min, const TVector<T>& Max) const
{
    return TVector<T>
    (
        FMath::Clamp(X, Min.X, Max.X),
        FMath::Clamp(Y, Min.Y, Max.Y),
        FMath::Clamp(Z, Min.Z, Max.Z)
    );
}


template<typename T>
inline TVector<T> TVector<T>::GetClampedToSize(T Min, T Max) const
{
    T VecSize = Size();
    const TVector<T> VecDir = (VecSize > UE_SMALL_NUMBER) ? (*this/VecSize) : ZeroVector;

    VecSize = FMath::Clamp(VecSize, Min, Max);

    return VecSize * VecDir;
}

template<typename T>
inline TVector<T> TVector<T>::GetClampedToSize2D(T Min, T Max) const
{
    T VecSize2D = Size2D();
    const TVector<T> VecDir = (VecSize2D > UE_SMALL_NUMBER) ? (*this/VecSize2D) : ZeroVector;

    VecSize2D = FMath::Clamp(VecSize2D, Min, Max);

    return TVector<T>(VecSize2D * VecDir.X, VecSize2D * VecDir.Y, Z);
}

template<typename T>
inline TVector<T> TVector<T>::GetClampedToMaxSize(T MaxSize) const
{
    if (MaxSize < UE_KINDA_SMALL_NUMBER)
    {
        return ZeroVector;
    }

    const T VSq = SizeSquared();
    if (VSq > FMath::Square(MaxSize))
    {
        const T Scale = MaxSize * FMath::InvSqrt(VSq);
        return TVector<T>(X*Scale, Y*Scale, Z*Scale);
    }
    else
    {
        return *this;
    }	
}

template<typename T>
inline TVector<T> TVector<T>::GetClampedToMaxSize2D(T MaxSize) const
{
    if (MaxSize < UE_KINDA_SMALL_NUMBER)
    {
        return TVector<T>(0.f, 0.f, Z);
    }

    const T VSq2D = SizeSquared2D();
    if (VSq2D > FMath::Square(MaxSize))
    {
        const T Scale = MaxSize * FMath::InvSqrt(VSq2D);
        return TVector<T>(X*Scale, Y*Scale, Z);
    }
    else
    {
        return *this;
    }
}

template<typename T>
UE_FORCEINLINE_HINT void TVector<T>::AddBounded(const TVector<T>& V, T Radius)
{
    *this = (*this + V).BoundToCube(Radius);
}

template<typename T>
inline T& TVector<T>::Component(int32 Index)
{
	check(IsValidIndex(Index));
PRAGMA_DISABLE_DEPRECATION_WARNINGS
    return XYZ[Index];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

template<typename T>
inline T TVector<T>::Component(int32 Index) const
{
	check(IsValidIndex(Index));
PRAGMA_DISABLE_DEPRECATION_WARNINGS
    return XYZ[Index];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

template<typename T>
bool TVector<T>::IsValidIndex(int32 Index) const
{
	return (Index >= 0) && (Index < NumComponents);
}

template<typename T>
inline T TVector<T>::GetComponentForAxis(EAxis::Type Axis) const
{
    switch (Axis)
    {
    case EAxis::X:
        return X;
    case EAxis::Y:
        return Y;
    case EAxis::Z:
        return Z;
	case EAxis::None:
    default:
        return 0.f;
    }
}

template<typename T>
inline void TVector<T>::SetComponentForAxis(EAxis::Type Axis, T Component)
{
    switch (Axis)
    {
    case EAxis::X:
        X = Component;
        break;
    case EAxis::Y:
        Y = Component;
        break;
    case EAxis::Z:
        Z = Component;
        break;
	case EAxis::None:
	default:
		break;
    }
}

template<typename T>
inline TVector<T> TVector<T>::Reciprocal() const
{
    TVector<T> RecVector;
    if (X!=0.f)
    {
        RecVector.X = 1.f/X;
    }
    else 
    {
        RecVector.X = UE_BIG_NUMBER;
    }
    if (Y!=0.f)
    {
        RecVector.Y = 1.f/Y;
    }
    else 
    {
        RecVector.Y = UE_BIG_NUMBER;
    }
    if (Z!=0.f)
    {
        RecVector.Z = 1.f/Z;
    }
    else 
    {
        RecVector.Z = UE_BIG_NUMBER;
    }

    return RecVector;
}




template<typename T>
UE_FORCEINLINE_HINT bool TVector<T>::IsUniform(T Tolerance) const
{
    return AllComponentsEqual(Tolerance);
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TVector<T>::MirrorByVector(const TVector<T>& MirrorNormal) const
{
    return *this - MirrorNormal * (2.f * (*this | MirrorNormal));
}

template<typename T>
inline TVector<T> TVector<T>::GetSafeNormal(T Tolerance, const TVector<T>& ResultIfZero) const
{
    const T SquareSum = X*X + Y*Y + Z*Z;

    // Not sure if it's safe to add tolerance in there. Might introduce too many errors
    if(SquareSum == 1.f)
    {
        return *this;
    }		
    else if(SquareSum < Tolerance)
    {
        return ResultIfZero;
    }
    const T Scale = (T)FMath::InvSqrt(SquareSum);
    return TVector<T>(X*Scale, Y*Scale, Z*Scale);
}

template<typename T>
inline TVector<T> TVector<T>::GetSafeNormal2D(T Tolerance, const TVector<T>& ResultIfZero) const
{
    const T SquareSum = X*X + Y*Y;

    // Not sure if it's safe to add tolerance in there. Might introduce too many errors
    if(SquareSum == 1.f)
    {
        if(Z == 0.f)
        {
            return *this;
        }
        else
        {
            return TVector<T>(X, Y, 0.f);
        }
    }
    else if(SquareSum < Tolerance)
    {
        return ResultIfZero;
    }

    const T Scale = FMath::InvSqrt(SquareSum);
    return TVector<T>(X*Scale, Y*Scale, 0.f);
}

template<typename T>
inline T TVector<T>::CosineAngle2D(TVector<T> B) const
{
    TVector<T> A(*this);
    A.Z = 0.0f;
    B.Z = 0.0f;
    A.Normalize();
    B.Normalize();
    return A | B;
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TVector<T>::ProjectOnTo(const TVector<T>& A) const
{ 
    return (A * ((*this | A) / (A | A))); 
}

template<typename T>
UE_FORCEINLINE_HINT TVector<T> TVector<T>::ProjectOnToNormal(const TVector<T>& Normal) const
{
    return (Normal * (*this | Normal));
}


template<typename T>
void TVector<T>::GenerateClusterCenters(TArray<TVector<T>>& Clusters, const TArray<TVector<T>>& Points, int32 NumIterations, int32 NumConnectionsToBeValid)
{
	struct FClusterMovedHereToMakeCompile
	{
		TVector<T> ClusterPosAccum;
		int32 ClusterSize;
	};

	// Check we have >0 points and clusters
	if (Points.Num() == 0 || Clusters.Num() == 0)
	{
		return;
	}

	// Temp storage for each cluster that mirrors the order of the passed in Clusters array
	TArray<FClusterMovedHereToMakeCompile> ClusterData;
	ClusterData.AddZeroed(Clusters.Num());

	// Then iterate
	for (int32 ItCount = 0; ItCount < NumIterations; ItCount++)
	{
		// Classify each point - find closest cluster center
		for (int32 i = 0; i < Points.Num(); i++)
		{
			const TVector<T>& Pos = Points[i];

			// Iterate over all clusters to find closes one
			int32 NearestClusterIndex = INDEX_NONE;
			T NearestClusterDistSqr = UE_BIG_NUMBER;
			for (int32 j = 0; j < Clusters.Num(); j++)
			{
				const T DistSqr = (Pos - Clusters[j]).SizeSquared();
				if (DistSqr < NearestClusterDistSqr)
				{
					NearestClusterDistSqr = DistSqr;
					NearestClusterIndex = j;
				}
			}
			// Update its info with this point
			if (NearestClusterIndex != INDEX_NONE)
			{
				ClusterData[NearestClusterIndex].ClusterPosAccum += Pos;
				ClusterData[NearestClusterIndex].ClusterSize++;
			}
		}

		// All points classified - update cluster center as average of membership
		for (int32 i = 0; i < Clusters.Num(); i++)
		{
			if (ClusterData[i].ClusterSize > 0)
			{
				Clusters[i] = ClusterData[i].ClusterPosAccum / (T)ClusterData[i].ClusterSize;
			}
		}
	}

	// so now after we have possible cluster centers we want to remove the ones that are outliers and not part of the main cluster
	for (int32 i = 0; i < ClusterData.Num(); i++)
	{
		if (ClusterData[i].ClusterSize < NumConnectionsToBeValid)
		{
			Clusters.RemoveAt(i);
		}
	}
}

template<typename T>
T TVector<T>::EvaluateBezier(const TVector<T>* ControlPoints, int32 NumPoints, TArray<TVector<T>>& OutPoints)
{
	check(ControlPoints);
	check(NumPoints >= 2);

	// var q is the change in t between successive evaluations.
	const T q = 1.f / (T)(NumPoints - 1); // q is dependent on the number of GAPS = POINTS-1

	// recreate the names used in the derivation
	const TVector<T>& P0 = ControlPoints[0];
	const TVector<T>& P1 = ControlPoints[1];
	const TVector<T>& P2 = ControlPoints[2];
	const TVector<T>& P3 = ControlPoints[3];

	// coefficients of the cubic polynomial that we're FDing -
	const TVector<T> a = P0;
	const TVector<T> b = 3 * (P1 - P0);
	const TVector<T> c = 3 * (P2 - 2 * P1 + P0);
	const TVector<T> d = P3 - 3 * P2 + 3 * P1 - P0;

	// initial values of the poly and the 3 diffs -
	TVector<T> S = a;						// the poly value
	TVector<T> U = b * q + c * q * q + d * q * q * q;	// 1st order diff (quadratic)
	TVector<T> V = 2 * c * q * q + 6 * d * q * q * q;	// 2nd order diff (linear)
	TVector<T> W = 6 * d * q * q * q;				// 3rd order diff (constant)

	// Path length.
	T Length = 0;

	TVector<T> OldPos = P0;
	OutPoints.Add(P0);	// first point on the curve is always P0.

	for (int32 i = 1; i < NumPoints; ++i)
	{
		// calculate the next value and update the deltas
		S += U;			// update poly value
		U += V;			// update 1st order diff value
		V += W;			// update 2st order diff value
		// 3rd order diff is constant => no update needed.

		// Update Length.
		Length += TVector<T>::Dist(S, OldPos);
		OldPos = S;

		OutPoints.Add(S);
	}

	// Return path length as experienced in sequence (linear interpolation between points).
	return Length;
}

template<typename T>
void TVector<T>::CreateOrthonormalBasis(TVector<T>& XAxis, TVector<T>& YAxis, TVector<T>& ZAxis)
{
	// Project the X and Y axes onto the plane perpendicular to the Z axis.
	XAxis -= (XAxis | ZAxis) / (ZAxis | ZAxis) * ZAxis;
	YAxis -= (YAxis | ZAxis) / (ZAxis | ZAxis) * ZAxis;

	// If the X axis was parallel to the Z axis, choose a vector which is orthogonal to the Y and Z axes.
	if (XAxis.SizeSquared() < UE_DELTA * UE_DELTA)
	{
		XAxis = YAxis ^ ZAxis;
	}

	// If the Y axis was parallel to the Z axis, choose a vector which is orthogonal to the X and Z axes.
	if (YAxis.SizeSquared() < UE_DELTA * UE_DELTA)
	{
		YAxis = XAxis ^ ZAxis;
	}

	// Normalize the basis vectors.
	XAxis.Normalize();
	YAxis.Normalize();
	ZAxis.Normalize();
}

template<typename T>
void TVector<T>::UnwindEuler()
{
	X = FMath::UnwindDegrees(X);
	Y = FMath::UnwindDegrees(Y);
	Z = FMath::UnwindDegrees(Z);
}

template<typename T>
void TVector<T>::FindBestAxisVectors(TVector<T>& Axis1, TVector<T>& Axis2) const
{
	const T NX = FMath::Abs(X);
	const T NY = FMath::Abs(Y);
	const T NZ = FMath::Abs(Z);

	// Find best basis vectors.
	if (NZ > NX && NZ > NY)	Axis1 = TVector<T>(1, 0, 0);
	else					Axis1 = TVector<T>(0, 0, 1);

	TVector<T> Tmp = Axis1 - *this * (Axis1 | *this);
	Axis1 = Tmp.GetSafeNormal();
	Axis2 = Axis1 ^ *this;
}

template<typename T>
inline bool TVector<T>::ContainsNaN() const
{
    return (!FMath::IsFinite(X) || 
            !FMath::IsFinite(Y) ||
            !FMath::IsFinite(Z));
}

template<typename T>
UE_FORCEINLINE_HINT FString TVector<T>::ToString() const
{
    return FString::Printf(TEXT("X=%3.3f Y=%3.3f Z=%3.3f"), X, Y, Z);
}

template<typename T>
inline FText TVector<T>::ToText() const
{
    FFormatNamedArguments Args;
    Args.Add(TEXT("X"), X);
    Args.Add(TEXT("Y"), Y);
    Args.Add(TEXT("Z"), Z);

    return FText::Format(NSLOCTEXT("Core", "Vector3", "X={X} Y={Y} Z={Z}"), Args);
}

template<typename T>
inline FText TVector<T>::ToCompactText() const
{
    if (IsNearlyZero())
    {
        return NSLOCTEXT("Core", "Vector3_CompactZeroVector", "V(0)");
    }

    const bool XIsNotZero = !FMath::IsNearlyZero(X);
    const bool YIsNotZero = !FMath::IsNearlyZero(Y);
    const bool ZIsNotZero = !FMath::IsNearlyZero(Z);

    FNumberFormattingOptions FormatRules;
    FormatRules.MinimumFractionalDigits = 2;
    FormatRules.MinimumIntegralDigits = 0;

    FFormatNamedArguments Args;
    Args.Add(TEXT("X"), FText::AsNumber(X, &FormatRules));
    Args.Add(TEXT("Y"), FText::AsNumber(Y, &FormatRules));
    Args.Add(TEXT("Z"), FText::AsNumber(Z, &FormatRules));

    if (XIsNotZero && YIsNotZero && ZIsNotZero)
    {
        return FText::Format(NSLOCTEXT("Core", "Vector3_CompactXYZ", "V(X={X}, Y={Y}, Z={Z})"), Args);
    }
    else if (!XIsNotZero && YIsNotZero && ZIsNotZero)
    {
        return FText::Format(NSLOCTEXT("Core", "Vector3_CompactYZ", "V(Y={Y}, Z={Z})"), Args);
    }
    else if (XIsNotZero && !YIsNotZero && ZIsNotZero)
    {
        return FText::Format(NSLOCTEXT("Core", "Vector3_CompactXZ", "V(X={X}, Z={Z})"), Args);
    }
    else if (XIsNotZero && YIsNotZero && !ZIsNotZero)
    {
        return FText::Format(NSLOCTEXT("Core", "Vector3_CompactXY", "V(X={X}, Y={Y})"), Args);
    }
    else if (!XIsNotZero && !YIsNotZero && ZIsNotZero)
    {
        return FText::Format(NSLOCTEXT("Core", "Vector3_CompactZ", "V(Z={Z})"), Args);
    }
    else if (XIsNotZero && !YIsNotZero && !ZIsNotZero)
    {
        return FText::Format(NSLOCTEXT("Core", "Vector3_CompactX", "V(X={X})"), Args);
    }
    else if (!XIsNotZero && YIsNotZero && !ZIsNotZero)
    {
        return FText::Format(NSLOCTEXT("Core", "Vector3_CompactY", "V(Y={Y})"), Args);
    }

    return NSLOCTEXT("Core", "Vector3_CompactZeroVector", "V(0)");
}

template<typename T>
inline FString TVector<T>::ToCompactString() const
{
    if(IsNearlyZero())
    {
        return FString("V(0)");
    }

    FString ReturnString("V(");
    bool bIsEmptyString = true;
    if(!FMath::IsNearlyZero(X))
    {
        ReturnString += FString::Printf(TEXT("X=%.2f"), X);
        bIsEmptyString = false;
    }
    if(!FMath::IsNearlyZero(Y))
    {
        if(!bIsEmptyString)
        {
            ReturnString += FString(", ");
        }
        ReturnString += FString::Printf(TEXT("Y=%.2f"), Y);
        bIsEmptyString = false;
    }
    if(!FMath::IsNearlyZero(Z))
    {
        if(!bIsEmptyString)
        {
            ReturnString += FString(", ");
        }
        ReturnString += FString::Printf(TEXT("Z=%.2f"), Z);
        bIsEmptyString = false;
    }
    ReturnString += FString(")");
    return ReturnString;
}

template<typename T>
inline bool TVector<T>::InitFromCompactString(const FString& InSourceString)
{
	X = Y = Z = 0;

	if (FCString::Strifind(*InSourceString, TEXT("V(0)")) != nullptr)
	{
		return true;
	}

	const bool bSuccessful = FParse::Value(*InSourceString, TEXT("X="), X) | FParse::Value(*InSourceString, TEXT("Y="), Y) | FParse::Value(*InSourceString, TEXT("Z="), Z); //-V792

	return bSuccessful;
}

template<typename T>
inline bool TVector<T>::InitFromString(const FString& InSourceString)
{
	X = Y = Z = 0;

	// The initialization is only successful if the X, Y, and Z values can all be parsed from the string
	const bool bSuccessful = FParse::Value(*InSourceString, TEXT("X=") , X) && FParse::Value(*InSourceString, TEXT("Y="), Y) && FParse::Value(*InSourceString, TEXT("Z="), Z);

	return bSuccessful;
}

template<typename T>
inline TVector2<T> TVector<T>::UnitCartesianToSpherical() const
{
    checkSlow(IsUnit());
    const T Theta = FMath::Acos(Z / Size());
    const T Phi = FMath::Atan2(Y, X);
    return TVector2<T>(Theta, Phi);
}

template<typename T>
inline T TVector<T>::HeadingAngle() const
{
    // Project Dir into Z plane.
    TVector<T> PlaneDir = *this;
    PlaneDir.Z = 0.f;
    PlaneDir = PlaneDir.GetSafeNormal();

    T Angle = FMath::Acos(PlaneDir.X);

    if(PlaneDir.Y < 0.0f)
    {
        Angle *= -1.0f;
    }

    return Angle;
}



template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::Dist(const TVector<T>& V1, const TVector<T>& V2)
{
    return FMath::Sqrt(TVector<T>::DistSquared(V1, V2));
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::DistXY(const TVector<T>& V1, const TVector<T>& V2)
{
    return FMath::Sqrt(TVector<T>::DistSquaredXY(V1, V2));
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::DistSquared(const TVector<T>& V1, const TVector<T>& V2)
{
    return FMath::Square(V2.X-V1.X) + FMath::Square(V2.Y-V1.Y) + FMath::Square(V2.Z-V1.Z);
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::DistSquaredXY(const TVector<T>& V1, const TVector<T>& V2)
{
    return FMath::Square(V2.X-V1.X) + FMath::Square(V2.Y-V1.Y);
}

template<typename T>
UE_FORCEINLINE_HINT T TVector<T>::BoxPushOut(const TVector<T>& Normal, const TVector<T>& Size)
{
    return FMath::Abs(Normal.X*Size.X) + FMath::Abs(Normal.Y*Size.Y) + FMath::Abs(Normal.Z*Size.Z);
}

template<typename T>
inline TVector<T> TVector<T>::Min(const TVector<T>& A, const TVector<T>& B)
{
    return TVector<T>(
        FMath::Min( A.X, B.X ),
        FMath::Min( A.Y, B.Y ),
        FMath::Min( A.Z, B.Z )
        ); 
}

template<typename T>
inline TVector<T> TVector<T>::Max(const TVector<T>& A, const TVector<T>& B)
{
    return TVector<T>(
        FMath::Max( A.X, B.X ),
        FMath::Max( A.Y, B.Y ),
        FMath::Max( A.Z, B.Z )
        ); 
}

template<typename T>
inline TVector<T> TVector<T>::Min3( const TVector<T>& A, const TVector<T>& B, const TVector<T>& C )
{
    return TVector<T>(
        FMath::Min3( A.X, B.X, C.X ),
        FMath::Min3( A.Y, B.Y, C.Y ),
        FMath::Min3( A.Z, B.Z, C.Z )
        ); 
}

template<typename T>
inline TVector<T> TVector<T>::Max3(const TVector<T>& A, const TVector<T>& B, const TVector<T>& C)
{
    return TVector<T>(
        FMath::Max3( A.X, B.X, C.X ),
        FMath::Max3( A.Y, B.Y, C.Y ),
        FMath::Max3( A.Z, B.Z, C.Z )
        ); 
}

#if !defined(_MSC_VER) || defined(__clang__)  // MSVC can't forward declare explicit specializations
template<> CORE_API const FVector3f FVector3f::ZeroVector;
template<> CORE_API const FVector3f FVector3f::OneVector;
template<> CORE_API const FVector3f FVector3f::UpVector;
template<> CORE_API const FVector3f FVector3f::DownVector;
template<> CORE_API const FVector3f FVector3f::ForwardVector;
template<> CORE_API const FVector3f FVector3f::BackwardVector;
template<> CORE_API const FVector3f FVector3f::RightVector;
template<> CORE_API const FVector3f FVector3f::LeftVector;
template<> CORE_API const FVector3f FVector3f::XAxisVector;
template<> CORE_API const FVector3f FVector3f::YAxisVector;
template<> CORE_API const FVector3f FVector3f::ZAxisVector;
template<> CORE_API const FVector3d FVector3d::ZeroVector;
template<> CORE_API const FVector3d FVector3d::OneVector;
template<> CORE_API const FVector3d FVector3d::UpVector;
template<> CORE_API const FVector3d FVector3d::DownVector;
template<> CORE_API const FVector3d FVector3d::ForwardVector;
template<> CORE_API const FVector3d FVector3d::BackwardVector;
template<> CORE_API const FVector3d FVector3d::RightVector;
template<> CORE_API const FVector3d FVector3d::LeftVector;
template<> CORE_API const FVector3d FVector3d::XAxisVector;
template<> CORE_API const FVector3d FVector3d::YAxisVector;
template<> CORE_API const FVector3d FVector3d::ZAxisVector;
#endif

/**
 * Multiplies a vector by a scaling factor.
 *
 * @param Scale Scaling factor.
 * @param V Vector to scale.
 * @return Result of multiplication.
 */
template<typename T, typename T2 UE_REQUIRES(std::is_arithmetic_v<T2>)>
UE_FORCEINLINE_HINT TVector<T> operator*(T2 Scale, const TVector<T>& V)
{
	return V.operator*(Scale);
}

/**
 * Creates a hash value from an FVector.
 *
 * @param Vector the vector to create a hash value for
 * @return The hash value from the components
 */
template<typename T>
UE_FORCEINLINE_HINT uint32 GetTypeHash(const TVector<T>& Vector)
{
	// Note: this assumes there's no padding in Vector that could contain uncompared data.
	return FCrc::MemCrc_DEPRECATED(&Vector, sizeof(Vector));
}

} // namespace UE::Math
} // namespace UE

template<> struct TCanBulkSerialize<FVector3f> { enum { Value = true }; };
template<> struct TIsPODType<FVector3f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FVector3f> { enum { Value = true }; };
DECLARE_INTRINSIC_TYPE_LAYOUT(FVector3f);

template<> struct TCanBulkSerialize<FVector3d> { enum { Value = true }; };
template<> struct TIsPODType<FVector3d> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FVector3d> { enum { Value = true }; };
DECLARE_INTRINSIC_TYPE_LAYOUT(FVector3d);
 
/** Component-wise clamp for TVector */
template<typename T>
inline UE::Math::TVector<T> ClampVector(const UE::Math::TVector<T>& V, const UE::Math::TVector<T>& Min, const UE::Math::TVector<T>& Max)
{
	return UE::Math::TVector<T>(
		FMath::Clamp(V.X, Min.X, Max.X),
		FMath::Clamp(V.Y, Min.Y, Max.Y),
		FMath::Clamp(V.Z, Min.Z, Max.Z)
	);
}

#if PLATFORM_LITTLE_ENDIAN
#define INTEL_ORDER_VECTOR(x) (x)
#else
template<typename T>
static UE_FORCEINLINE_HINT UE::Math::TVector<T> INTEL_ORDER_VECTOR(UE::Math::TVector<T> v)
{
	return UE::Math::TVector<T>(INTEL_ORDERF(v.X), INTEL_ORDERF(v.Y), INTEL_ORDERF(v.Z));
}
#endif


/**
 * Util to calculate distance from a point to a bounding box
 *
 * @param Mins 3D Point defining the lower values of the axis of the bound box
 * @param Max 3D Point defining the lower values of the axis of the bound box
 * @param Point 3D position of interest
 * @return the distance from the Point to the bounding box.
 */
template<typename T, typename U>
inline T ComputeSquaredDistanceFromBoxToPoint(const UE::Math::TVector<T>& Mins, const UE::Math::TVector<T>& Maxs, const UE::Math::TVector<U>& Point)
{
	// Accumulates the distance as we iterate axis
	T DistSquared = 0;

	// Check each axis for min/max and add the distance accordingly
	// NOTE: Loop manually unrolled for > 2x speed up
	if (Point.X < Mins.X)
	{
		DistSquared += FMath::Square(Point.X - Mins.X);
	}
	else if (Point.X > Maxs.X)
	{
		DistSquared += FMath::Square(Point.X - Maxs.X);
	}

	if (Point.Y < Mins.Y)
	{
		DistSquared += FMath::Square(Point.Y - Mins.Y);
	}
	else if (Point.Y > Maxs.Y)
	{
		DistSquared += FMath::Square(Point.Y - Maxs.Y);
	}

	if (Point.Z < Mins.Z)
	{
		DistSquared += FMath::Square(Point.Z - Mins.Z);
	}
	else if (Point.Z > Maxs.Z)
	{
		DistSquared += FMath::Square(Point.Z - Maxs.Z);
	}

	return DistSquared;
}


/* FMath inline functions
 *****************************************************************************/

template<typename T>
inline UE::Math::TVector<T> FMath::LinePlaneIntersection
    (
    const UE::Math::TVector<T>&Point1,
    const UE::Math::TVector<T>&Point2,
    const UE::Math::TVector<T>&PlaneOrigin,
    const UE::Math::TVector<T>&PlaneNormal
    )
{
    return
        Point1
        + (Point2 - Point1)
        *	(((PlaneOrigin - Point1)|PlaneNormal) / ((Point2 - Point1)|PlaneNormal));
}

template<typename T>
inline bool FMath::LineSphereIntersection(const UE::Math::TVector<T>& Start, const UE::Math::TVector<T>& Dir, T Length, const UE::Math::TVector<T>& Origin, T Radius)
{
    const UE::Math::TVector<T>	EO = Start - Origin;
    const T		v = (Dir | (Origin - Start));
    const T		disc = Radius * Radius - ((EO | EO) - v * v);

    if(disc >= 0)
    {
        const T	Time = (v - Sqrt(disc)) / Length;

        if(Time >= 0 && Time <= 1)
            return 1;
        else
            return 0;
    }
    else
        return 0;
}

inline FVector FMath::VRand()
{
    FVector Result;
    FVector::FReal L;

    do
    {
        // Check random vectors in the unit sphere so result is statistically uniform.
        Result.X = FRand() * 2.f - 1.f;
        Result.Y = FRand() * 2.f - 1.f;
        Result.Z = FRand() * 2.f - 1.f;
        L = Result.SizeSquared();
    }
    while(L > 1.0f || L < UE_KINDA_SMALL_NUMBER);

    return Result * (1.0f / Sqrt(L));
}

/* TVector2<T> inline functions
 *****************************************************************************/
namespace UE {
namespace Math {

template<typename InIntType>
template <typename FloatType>
TIntVector3<InIntType>::TIntVector3(TVector<FloatType> InVector)
{
	if constexpr (std::is_same_v<IntType, int32>)
	{
		X = FMath::TruncToInt32(InVector.X);
		Y = FMath::TruncToInt32(InVector.Y);
		Z = FMath::TruncToInt32(InVector.Z);
	}
	else if constexpr (std::is_same_v<IntType, uint32>)
	{
		X = IntCastChecked<uint32, int64>(FMath::TruncToInt64(InVector.X));
		Y = IntCastChecked<uint32, int64>(FMath::TruncToInt64(InVector.Y));
		Z = IntCastChecked<uint32, int64>(FMath::TruncToInt64(InVector.Z));
	}
	else
	{
		static_assert(sizeof(IntType) == 0, "Unimplemented");
	}
}

template<typename T>
inline TVector2<T>::TVector2( const TVector<T>& V )
	: X(V.X), Y(V.Y)
{
	DiagnosticCheckNaN();
}

template<typename T>
inline TVector<T> TVector2<T>::SphericalToUnitCartesian() const
{
    const T SinTheta = FMath::Sin(X);
    return TVector<T>(FMath::Cos(Y) * SinTheta, FMath::Sin(Y) * SinTheta, FMath::Cos(X));
}

} // namespace UE::Math
	
namespace LWC
{
inline constexpr FVector::FReal DefaultFloatPrecision = 1./16.;

// Validated narrowing cast for world positions. FVector -> FVector3f
inline FVector3f NarrowWorldPositionChecked(const FVector& WorldPosition)
{
	FVector3f Narrowed;
	Narrowed.X = FloatCastChecked<float>(WorldPosition.X, DefaultFloatPrecision);
	Narrowed.Y = FloatCastChecked<float>(WorldPosition.Y, DefaultFloatPrecision);
	Narrowed.Z = FloatCastChecked<float>(WorldPosition.Z, DefaultFloatPrecision);
	return Narrowed;
}

// Validated narrowing cast for world positions. FVector -> FVector3f
inline FVector3f NarrowWorldPositionChecked(const FVector::FReal InX, const FVector::FReal InY, const FVector::FReal InZ)
{
	FVector3f Narrowed;
	Narrowed.X = FloatCastChecked<float>(InX, DefaultFloatPrecision);
	Narrowed.Y = FloatCastChecked<float>(InY, DefaultFloatPrecision);
	Narrowed.Z = FloatCastChecked<float>(InZ, DefaultFloatPrecision);
	return Narrowed;
}

} // namespace UE::LWC

} // namespace UE

#if PLATFORM_ENABLE_VECTORINTRINSICS
template<>
inline FVector FMath::CubicInterp(const FVector& P0, const FVector& T0, const FVector& P1, const FVector& T1, const float& A)
{
	static_assert(PLATFORM_ENABLE_VECTORINTRINSICS == 1, "Requires vector intrinsics.");
	FVector res;

	const float A2 = A * A;
	const float A3 = A2 * A;

	const float s0 = (2 * A3) - (3 * A2) + 1;
	const float s1 = A3 - (2 * A2) + A;
	const float s2 = (A3 - A2);
	const float s3 = (-2 * A3) + (3 * A2);

	VectorRegister v0 = VectorMultiply(VectorLoadFloat1(&s0), VectorLoadFloat3(&P0));
	v0 = VectorMultiplyAdd(VectorLoadFloat1(&s1), VectorLoadFloat3(&T0), v0);
	VectorRegister v1 = VectorMultiply(VectorLoadFloat1(&s2), VectorLoadFloat3(&T1));
	v1 = VectorMultiplyAdd(VectorLoadFloat1(&s3), VectorLoadFloat3(&P1), v1);

	VectorStoreFloat3(VectorAdd(v0, v1), &res);

	return res;
}
#endif

#ifdef _MSC_VER
#pragma warning (pop)
#endif
