// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __VECTOR_ISPH__
#define __VECTOR_ISPH__

#include "Math/Aossoa.isph"
#include "Math/Soaaos.isph"
#include "Math/Scalar.isph"

// Opt-in for explicit vector structs. Can assist with alignment and codegen in some cases.
#ifndef EXPLICIT_VECTOR2
#define EXPLICIT_VECTOR2 0
#endif

// Opt-out of VectorGather optimization
#ifndef USE_ISPC_OPTIMIZED_VECTORGATHER
#define USE_ISPC_OPTIMIZED_VECTORGATHER 1
#endif

// Opt-out of VectorScatter optimization
#ifndef USE_ISPC_OPTIMIZED_VECTORSCATTER
#define USE_ISPC_OPTIMIZED_VECTORSCATTER 1
#endif

struct FVector2f
{
#if EXPLICIT_VECTOR2 == 0
	float V[2];
#else
	float<2> V;
#endif
};

struct FVector2
{
	double V[2];
};

#ifndef EXPLICIT_VECTOR
#define EXPLICIT_VECTOR 0
#endif

struct FVector3f
{
#if EXPLICIT_VECTOR == 0
	float V[3];
#else
	float<3> V;
#endif
};

struct FVector
{
	double V[3];
};

#ifndef EXPLICIT_VECTOR4
#define EXPLICIT_VECTOR4 0
#endif

struct FVector4f
{
#if EXPLICIT_VECTOR4 == 0
	float V[4];
#else
	float<4> V;
#endif
};

struct FVector4
{
	double V[4];
};

struct FVector8
{
#if EXPLICIT_VECTOR8 == 0
	float V[8];
#else
	float<8> V;
#endif
};

#ifndef EXPLICIT_INTVECTOR2
#define EXPLICIT_INTVECTOR2 0
#endif

struct FIntVector2
{
#if EXPLICIT_INTVECTOR2 == 0
	int V[2];
#else
	int<2> V;
#endif
};

#ifndef EXPLICIT_UINTVECTOR2
#define EXPLICIT_UINTVECTOR2 0
#endif

struct FUIntVector2
{
#if EXPLICIT_UINTVECTOR2 == 0
	uint V[2];
#else
	uint<2> V;
#endif
};

#ifndef EXPLICIT_INTVECTOR
#define EXPLICIT_INTVECTOR 0
#endif

struct FIntVector
{
#if EXPLICIT_INTVECTOR == 0
	int V[3];
#else
	int<3> V;
#endif
};

#ifndef EXPLICIT_INTVECTOR4
#define EXPLICIT_INTVECTOR4 0
#endif

struct FIntVector4
{
#if EXPLICIT_INTVECTOR4 == 0
	int V[4];
#else
	int<4> V;
#endif
};

// Define types
typedef FVector2 FVector2d;
typedef FVector FVector3d;
typedef FVector4 FVector4d;

static const uniform struct FVector4f FLOAT_QMULTI_SIGN_MASK0 = { {1.f, -1.f, 1.f, -1.f} };
static const uniform struct FVector4f FLOAT_QMULTI_SIGN_MASK1 = { {1.f, 1.f, -1.f, -1.f} };
static const uniform struct FVector4f FLOAT_QMULTI_SIGN_MASK2 = { {-1.f, 1.f, 1.f, -1.f} };
static const uniform struct FVector4f FLOAT_QINV_SIGN_MASK = { {-1.f, -1.f, -1.f, 1.f} };
static const uniform struct FVector4f FloatZero = { {0.f, 0.f, 0.f, 0.f} };
static const uniform struct FVector4f FloatOne = { {1.f, 1.f, 1.f, 1.f} };
static const uniform struct FVector4f FloatMinusOne = { {-1.f, -1.f, -1.f, -1.f} };
static const uniform struct FVector4f Float0001 = { {0.0f, 0.0f, 0.0f, 1.0f} };
static const uniform struct FVector4f Float1110 = { {1.0f, 1.0f, 1.0f, 0.0f} };
static const uniform struct FVector4f FloatOneHalf = { {0.5f, 0.5f, 0.5f, 0.5f} };
static const uniform struct FVector4f FloatSmallLengthThreshold = { {FLOAT_SMALL_NUMBER, FLOAT_SMALL_NUMBER, FLOAT_SMALL_NUMBER, FLOAT_SMALL_NUMBER} };

static const uniform struct FVector4d DOUBLE_QMULTI_SIGN_MASK0 = { {1.d, -1.d, 1.d, -1.d} };
static const uniform struct FVector4d DOUBLE_QMULTI_SIGN_MASK1 = { {1.d, 1.d, -1.d, -1.d} };
static const uniform struct FVector4d DOUBLE_QMULTI_SIGN_MASK2 = { {-1.d, 1.d, 1.d, -1.d} };
static const uniform struct FVector4d DOUBLE_QINV_SIGN_MASK = { {-1.d, -1.d, -1.d, 1.d} };
static const uniform struct FVector4d DoubleZero = { {0.d, 0.d, 0.d, 0.d} };
static const uniform struct FVector4d DoubleOne = { {1.d, 1.d, 1.d, 1.d} };
static const uniform struct FVector4d DoubleMinusOne = { {-1.d, -1.d, -1.d, -1.d} };
static const uniform struct FVector4d Double0001 = { {0.0d, 0.0d, 0.0d, 1.0d} };
static const uniform struct FVector4d Double1110 = { {1.0d, 1.0d, 1.0d, 0.0d} };
static const uniform struct FVector4d DoubleOneHalf = { {0.5d, 0.5d, 0.5d, 0.5d} };
static const uniform struct FVector4d DoubleSmallLengthThreshold = { {DOUBLE_SMALL_NUMBER, DOUBLE_SMALL_NUMBER, DOUBLE_SMALL_NUMBER, DOUBLE_SMALL_NUMBER} };

static const uniform struct FVector4f FloatPi = { {FLOAT_PI, FLOAT_PI, FLOAT_PI, FLOAT_PI} };
static const uniform struct FVector4f FloatTwoPi = { {2.0f*FLOAT_PI, 2.0f*FLOAT_PI, 2.0f*FLOAT_PI, 2.0f*FLOAT_PI} };
static const uniform struct FVector4f FloatPiByTwo = { {0.5f*FLOAT_PI, 0.5f*FLOAT_PI, 0.5f*FLOAT_PI, 0.5f*FLOAT_PI} };
static const uniform struct FVector4f FloatPiByFour = { {0.25f*FLOAT_PI, 0.25f*FLOAT_PI, 0.25f*FLOAT_PI, 0.25f*FLOAT_PI} };
static const uniform struct FVector4f FloatOneOverPi = { {1.0f / FLOAT_PI, 1.0f / FLOAT_PI, 1.0f / FLOAT_PI, 1.0f / FLOAT_PI} };
static const uniform struct FVector4f FloatOneOverTwoPi = { {1.0f / (2.0f*FLOAT_PI), 1.0f / (2.0f*FLOAT_PI), 1.0f / (2.0f*FLOAT_PI), 1.0f / (2.0f*FLOAT_PI)} };

static const uniform struct FVector4d DoublePi = { {DOUBLE_PI, DOUBLE_PI, DOUBLE_PI, DOUBLE_PI} };
static const uniform struct FVector4d DoubleTwoPi = { {2.0d*DOUBLE_PI, 2.0d*DOUBLE_PI, 2.0d*DOUBLE_PI, 2.0d*DOUBLE_PI} };
static const uniform struct FVector4d DoublePiByTwo = { {0.5d*DOUBLE_PI, 0.5d*DOUBLE_PI, 0.5d*DOUBLE_PI, 0.5d*DOUBLE_PI} };
static const uniform struct FVector4d DoublePiByFour = { {0.25d*DOUBLE_PI, 0.25d*DOUBLE_PI, 0.25d*DOUBLE_PI, 0.25d*DOUBLE_PI} };
static const uniform struct FVector4d DoubleOneOverPi = { {1.0d / DOUBLE_PI, 1.0d / DOUBLE_PI, 1.0d / DOUBLE_PI, 1.0d / DOUBLE_PI} };
static const uniform struct FVector4d DoubleOneOverTwoPi = { {1.0d / (2.0d*DOUBLE_PI), 1.0d / (2.0d*DOUBLE_PI), 1.0d / (2.0d*DOUBLE_PI), 1.0d / (2.0d*DOUBLE_PI)} };

static const uniform struct FVector4f Float255 = { {255.0f, 255.0f, 255.0f, 255.0f} };
static const uniform struct FVector4f Float127 = { {127.0f, 127.0f, 127.0f, 127.0f} };
static const uniform struct FVector4f FloatNeg127 = { {-127.0f, -127.0f, -127.0f, -127.0f} };
static const uniform struct FVector4f Float360 = { {360.f, 360.f, 360.f, 360.f} };
static const uniform struct FVector4f Float180 = { {180.f, 180.f, 180.f, 180.f} };
static const uniform struct FVector4f FloatNonFractional = { {FLOAT_NON_FRACTIONAL, FLOAT_NON_FRACTIONAL, FLOAT_NON_FRACTIONAL, FLOAT_NON_FRACTIONAL} };

static const uniform struct FVector4d Double255 = { {255.0d, 255.0d, 255.0d, 255.0d} };
static const uniform struct FVector4d Double127 = { {127.0d, 127.0d, 127.0d, 127.0d} };
static const uniform struct FVector4d DoubleNeg127 = { {-127.0d, -127.0d, -127.0d, -127.0d} };
static const uniform struct FVector4d Double360 = { {360.d, 360.d, 360.d, 360.d} };
static const uniform struct FVector4d Double180 = { {180.d, 180.d, 180.d, 180.d} };
static const uniform struct FVector4d DoubleNonFractional = { {DOUBLE_NON_FRACTIONAL, DOUBLE_NON_FRACTIONAL, DOUBLE_NON_FRACTIONAL, DOUBLE_NON_FRACTIONAL} };

static const uniform struct FVector4f FLOAT_DEG_TO_RAD ={ {PI/(180.f), PI/(180.f), PI/(180.f), PI/(180.f)} };
static const uniform struct FVector4f FLOAT_DEG_TO_RAD_HALF = { {(PI/180.f)*0.5f, (PI/180.f)*0.5f, (PI/180.f)*0.5f, (PI/180.f)*0.5f} };
static const uniform struct FVector4f FLOAT_RAD_TO_DEG = { {(180.f)/PI, (180.f)/PI, (180.f)/PI, (180.f)/PI} };

static const uniform struct FVector4d DOUBLE_DEG_TO_RAD ={ {DOUBLE_PI/(180.d), DOUBLE_PI/(180.d), DOUBLE_PI/(180.d), DOUBLE_PI/(180.d)} };
static const uniform struct FVector4d DOUBLE_DEG_TO_RAD_HALF = { {(DOUBLE_PI/180.d)*0.5d, (DOUBLE_PI/180.d)*0.5d, (DOUBLE_PI/180.d)*0.5d, (DOUBLE_PI/180.d)*0.5d} };
static const uniform struct FVector4d DOUBLE_RAD_TO_DEG = { {(180.d)/DOUBLE_PI, (180.d)/DOUBLE_PI, (180.d)/DOUBLE_PI, (180.d)/DOUBLE_PI} };

static const uniform struct FVector4f FloatUpVector4 = { {0.0f, 0.0f, 1.0f, 0.0f} };
static const uniform struct FVector4f FloatForwardVector4 = { {1.0f, 0.0f, 0.0f, 0.0f} };
static const uniform struct FVector4f FloatRightVector4 = { {0.0f, 1.0f, 0.0f, 0.0f} };

static const uniform struct FVector4d DoubleUpVector4 = { {0.0d, 0.0d, 1.0d, 0.0d} };
static const uniform struct FVector4d DoubleForwardVector4 = { {1.0d, 0.0d, 0.0d, 0.0d} };
static const uniform struct FVector4d DoubleRightVector4 = { {0.0d, 1.0d, 0.0d, 0.0d} };

static const uniform struct FVector3f FloatZeroVector = { {0.0f, 0.0f, 0.0f} };
static const uniform struct FVector3f FloatOneVector = { {1.0f, 1.0f, 1.0f} };
static const uniform struct FVector3f FloatHalfVector = { {0.5f, 0.5f, 0.5f} };
static const uniform struct FVector3f FloatUpVector = { {0.0f, 0.0f, 1.0f} };
static const uniform struct FVector3f FloatDownVector = { {0.0f, 0.0f, -1.0f} };
static const uniform struct FVector3f FloatForwardVector = { {1.0f, 0.0f, 0.0f} };
static const uniform struct FVector3f FloatBackwardVector = { {-1.0f, 0.0f, 0.0f} };
static const uniform struct FVector3f FloatRightVector = { {0.0f, 1.0f, 0.0f} };
static const uniform struct FVector3f FloatLeftVector = { {0.0f, -1.0f, 0.0f} };

static const uniform struct FVector3d DoubleZeroVector = { {0.0d, 0.0d, 0.0d} };
static const uniform struct FVector3d DoubleOneVector = { {1.0d, 1.0d, 1.0d} };
static const uniform struct FVector3d DoubleHalfVector = { {0.5d, 0.5d, 0.5d} };
static const uniform struct FVector3d DoubleUpVector = { {0.0d, 0.0d, 1.0d} };
static const uniform struct FVector3d DoubleDownVector = { {0.0d, 0.0d, -1.0d} };
static const uniform struct FVector3d DoubleForwardVector = { {1.0d, 0.0d, 0.0d} };
static const uniform struct FVector3d DoubleBackwardVector = { {-1.0d, 0.0d, 0.0d} };
static const uniform struct FVector3d DoubleRightVector = { {0.0d, 1.0d, 0.0d} };
static const uniform struct FVector3d DoubleLeftVector = { {0.0d, -1.0d, 0.0d} };

static const uniform struct FVector8 UpVector8 = { {0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, -1.0f, 0.0f} };
static const uniform struct FVector8 ForwardVector8 = { {1.0f, 0.0f, 0.0f, 0.0f, -1.0f, 0.0f, 0.0f, 0.0f} };
static const uniform struct FVector8 RightVector8 = { {0.0f, 1.0f, 0.0f, 0.0f, 0.0f, -1.0f, 0.0f, 0.0f} };

// Define vector type constants

#define VectorZero				DoubleZero
#define VectorOne				DoubleOne
#define VectorMinusOne			DoubleMinusOne
#define Vector0001				Double0001
#define Vector1110				Double1110
#define VectorOneHalf			DoubleOneHalf
#define SmallLengthThreshold	DoubleSmallLengthThreshold

#define VectorPi				DoublePi
#define VectorTwoPi				DoubleTwoPi
#define VectorPiByTwo			DoublePiByTwo
#define VectorPiByFour			DoublePiByFour
#define VectorOneOverPi			DoubleOneOverPi
#define VectorOneOverTwoPi		DoubleOneOverTwoPi

#define Vector255				Double255
#define Vector127				Double127
#define VectorNeg127			DoubleNeg127
#define Vector360				Double360
#define Vector180				Double180
#define VectorNonFractional		DoubleNonFractional

#define UpVector4				DoubleUpVector4
#define ForwardVector4			DoubleForwardVector4
#define RightVector4			DoubleRightVector4

#define ZeroVector				DoubleZeroVector
#define OneVector				DoubleOneVector
#define HalfVector				DoubleHalfVector
#define UpVector				DoubleUpVector
#define DownVector				DoubleDownVector
#define ForwardVector			DoubleForwardVector
#define BackwardVector			DoubleBackwardVector
#define RightVector				DoubleRightVector
#define LeftVector				DoubleLeftVector

#if (defined(ISPC_TARGET_AVX2) || defined(ISPC_TARGET_AVX512SKX))
#define HW_GATHER_SUPPORTED 1
#else
#define HW_GATHER_SUPPORTED 0
#endif

#if defined(ISPC_TARGET_AVX512SKX)
#define HW_SCATTER_SUPPORTED 1
#else
#define HW_SCATTER_SUPPORTED 0
#endif

inline varying FUIntVector2 VectorLoad(const uniform FUIntVector2 *uniform SrcPtr)
{
	varying FUIntVector2 Result;

	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		aos_to_soa2_ispc((uniform int32 * uniform)SrcPtr, (varying int32 *uniform)&Result.V[0], (varying int32 *uniform)&Result.V[1]);
	}
	else
	{
		#pragma ignore warning(perf)
		Result = SrcPtr[programIndex];
	}

	return Result;
}

inline void VectorStore(uniform FUIntVector2 *uniform DstPtr, const varying FUIntVector2 &V)
{
	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		soa_to_aos2_ispc(*((varying int32 *uniform)&V.V[0]), *((varying int32 *uniform)&V.V[1]), (uniform int32 *uniform)DstPtr);
	}
	else
	{
		#pragma ignore warning(perf)
		DstPtr[programIndex] = V;
	}
}

inline varying FIntVector2 VectorLoad(const uniform FIntVector2 *uniform SrcPtr)
{
	varying FIntVector2 Result;

	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		aos_to_soa2_ispc((uniform int32 * uniform)SrcPtr, &Result.V[0], &Result.V[1]);
	}
	else
	{
		#pragma ignore warning(perf)
		Result = SrcPtr[programIndex];
	}

	return Result;
}

inline void VectorStore(uniform FIntVector2 *uniform DstPtr, const varying FIntVector2 &V)
{
	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		soa_to_aos2_ispc(V.V[0], V.V[1], (uniform int32 *uniform)DstPtr);
	}
	else
	{
		#pragma ignore warning(perf)
		DstPtr[programIndex] = V;
	}
}

inline varying FIntVector VectorLoad(const uniform FIntVector *uniform SrcPtr)
{
	varying FIntVector Result;

	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		aos_to_soa3_ispc((uniform int32 * uniform)SrcPtr, &Result.V[0], &Result.V[1], &Result.V[2]);
	}
	else
	{
		#pragma ignore warning(perf)
		Result = SrcPtr[programIndex];
	}

	return Result;
}

inline void VectorStore(uniform FIntVector *uniform DstPtr, const varying FIntVector &V)
{
	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		soa_to_aos3_ispc(V.V[0], V.V[1], V.V[2], (uniform int32 *uniform)DstPtr);
	}
	else
	{
		#pragma ignore warning(perf)
		DstPtr[programIndex] = V;
	}
}

inline varying FIntVector4 VectorLoad(const uniform FIntVector4 *uniform SrcPtr)
{
	varying FIntVector4 Result;

	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		aos_to_soa4_ispc((uniform int32 * uniform)SrcPtr, &Result.V[0], &Result.V[1], &Result.V[2], &Result.V[3]);
	}
	else
	{
		#pragma ignore warning(perf)
		Result = SrcPtr[programIndex];
	}

	return Result;
}

inline void VectorStore(uniform FIntVector4 *uniform DstPtr, const varying FIntVector4 &V)
{
	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		soa_to_aos4_ispc(V.V[0], V.V[1], V.V[2], V.V[3], (uniform int32 *uniform)DstPtr);
	}
	else
	{
		#pragma ignore warning(perf)
		DstPtr[programIndex] = V;
	}
}


inline varying FVector2f VectorLoad(const uniform FVector2f *uniform SrcPtr)
{
	varying FVector2f Result;

	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		aos_to_soa2_ispc((uniform float * uniform)SrcPtr, &Result.V[0], &Result.V[1]);
	}
	else
	{
		#pragma ignore warning(perf)
		Result = SrcPtr[programIndex];
	}

	return Result;
}


inline varying FVector2 VectorLoad(const uniform FVector2 *uniform SrcPtr)
{
	varying FVector2 Result;

	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		aos_to_soa2_ispc((uniform FReal * uniform)SrcPtr, &Result.V[0], &Result.V[1]);
	}
	else
	{
		#pragma ignore warning(perf)
		Result = SrcPtr[programIndex];
	}

	return Result;
}

inline void VectorStore(uniform FVector2f* uniform DstPtr, const varying FVector2f& V)
{
	if (((1 << TARGET_WIDTH) - 1 ^ lanemask()) == 0)
	{
		soa_to_aos2_ispc(V.V[0], V.V[1], (uniform float* uniform)DstPtr);
	}
	else
	{
#pragma ignore warning(perf)
		DstPtr[programIndex] = V;
	}
}

inline void VectorStore(uniform FVector2 *uniform DstPtr, const varying FVector2 &V)
{
	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		soa_to_aos2_ispc(V.V[0], V.V[1], (uniform FReal *uniform)DstPtr);
	}
	else
	{
		#pragma ignore warning(perf)
		DstPtr[programIndex] = V;
	}
}

inline varying FVector3d VectorLoad(const uniform FVector3d *uniform SrcPtr)
{
	varying FVector3d Result;

	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		aos_to_soa3_ispc((uniform double * uniform)SrcPtr, &Result.V[0], &Result.V[1], &Result.V[2]);
	}
	else
	{
		#pragma ignore warning(perf)
		Result = SrcPtr[programIndex];
	}

	return Result;
}

inline varying FVector3f VectorLoad(const uniform FVector3f *uniform SrcPtr)
{
	varying FVector3f Result;

	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		aos_to_soa3_ispc((uniform float * uniform)SrcPtr, &Result.V[0], &Result.V[1], &Result.V[2]);
	}
	else
	{
		#pragma ignore warning(perf)
		Result = SrcPtr[programIndex];
	}

	return Result;
}

inline void VectorStore(uniform FVector3d *uniform DstPtr, const varying FVector3d &V)
{
	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		soa_to_aos3_ispc(V.V[0], V.V[1], V.V[2], (uniform double *uniform)DstPtr);
	}
	else
	{
		#pragma ignore warning(perf)
		DstPtr[programIndex] = V;
	}
}

inline void VectorStore(uniform FVector3f *uniform DstPtr, const varying FVector3f &V)
{
	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		soa_to_aos3_ispc(V.V[0], V.V[1], V.V[2], (uniform float *uniform)DstPtr);
	}
	else
	{
		#pragma ignore warning(perf)
		DstPtr[programIndex] = V;
	}
}

inline varying FVector4d VectorLoad(const uniform FVector4d *uniform SrcPtr)
{
	varying FVector4d Result;

	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		aos_to_soa4_ispc((uniform double * uniform)SrcPtr, &Result.V[0], &Result.V[1], &Result.V[2], &Result.V[3]);
	}
	else
	{
		#pragma ignore warning(perf)
		Result = SrcPtr[programIndex];
	}

	return Result;
}

inline varying FVector4f VectorLoad(const uniform FVector4f *uniform SrcPtr)
{
	varying FVector4f Result;

	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		aos_to_soa4_ispc((uniform float * uniform)SrcPtr, &Result.V[0], &Result.V[1], &Result.V[2], &Result.V[3]);
	}
	else
	{
		#pragma ignore warning(perf)
		Result = SrcPtr[programIndex];
	}

	return Result;
}

inline void VectorStore(uniform FVector4d *uniform DstPtr, const varying FVector4d &V)
{
	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		soa_to_aos4_ispc(V.V[0], V.V[1], V.V[2], V.V[3], (uniform double *uniform)DstPtr);
	}
	else
	{
		#pragma ignore warning(perf)
		DstPtr[programIndex] = V;
	}
}

inline void VectorStore(uniform FVector4f *uniform DstPtr, const varying FVector4f &V)
{
	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		soa_to_aos4_ispc(V.V[0], V.V[1], V.V[2], V.V[3], (uniform float *uniform)DstPtr);
	}
	else
	{
		#pragma ignore warning(perf)
		DstPtr[programIndex] = V;
	}
}

inline varying FVector2d VectorGather(const uniform FVector2d *varying SrcPtr)
{
	#pragma ignore warning(perf)
	return *SrcPtr;
}

inline varying FVector2f VectorGather(const uniform FVector2f *varying SrcPtr)
{
	#pragma ignore warning(perf)
	return *SrcPtr;
}

inline void VectorScatter(uniform FVector2d *varying DstPtr, const varying FVector2d &V)
{
	#pragma ignore warning(perf)
	*DstPtr = V;
}

inline void VectorScatter(uniform FVector2f *varying DstPtr, const varying FVector2f &V)
{
	#pragma ignore warning(perf)
	*DstPtr = V;
}

inline varying FVector3d VectorGather(const uniform FVector3d *varying SrcPtr)
{
	#pragma ignore warning(perf)
	return *SrcPtr;
}

inline varying FVector3f VectorGather(const uniform FVector3f *varying SrcPtr)
{
	varying FVector3f Result;

#if USE_ISPC_OPTIMIZED_VECTORGATHER
	if (((1 << TARGET_WIDTH) - 1 ^ lanemask()) == 0)
	{
#if TARGET_WIDTH == 4
		AosToSoa3Explicit((uniform float* varying)SrcPtr, (uniform float* uniform)&Result);
#else
		UniformLoad(SrcPtr, (uniform FVector3f* uniform)&Result);
		aos_to_soa3_ispc((uniform float* uniform)&Result, &Result.V[0], &Result.V[1], &Result.V[2]);
#endif
	}
	else
#endif
	{
		#pragma ignore warning(perf)
		Result = *SrcPtr;
	}

	return Result;
}

inline void VectorScatter(uniform FVector3d *varying DstPtr, const varying FVector3d &V)
{
#if USE_ISPC_OPTIMIZED_VECTORSCATTER
	if (((1 << TARGET_WIDTH) - 1 ^ lanemask()) == 0)
	{
		unmasked
		{
#if TARGET_WIDTH == 4
			SoaToAos3Explicit((uniform double* uniform)&V, (uniform double* varying)DstPtr);
#else
			uniform FVector3d Result[programCount];
			soa_to_aos3_ispc(V.V[0], V.V[1], V.V[2], (uniform double* uniform)&Result);
			UniformStore(Result, DstPtr);
#endif
		}
	}
	else
#endif
	{
		#pragma ignore warning(perf)
		*DstPtr = V;
	}
}

inline void VectorScatter(uniform FVector3f *varying DstPtr, const varying FVector3f &V)
{
#if USE_ISPC_OPTIMIZED_VECTORSCATTER
	if (((1 << TARGET_WIDTH) - 1 ^ lanemask()) == 0)
	{
		unmasked
		{
#if TARGET_WIDTH == 4
			SoaToAos3Explicit((uniform float* uniform)&V, (uniform float* varying)DstPtr);
#else
			uniform FVector3f Result[programCount];
			soa_to_aos3_ispc(V.V[0], V.V[1], V.V[2], (uniform float* uniform)&Result);
			UniformStore(Result, DstPtr);
#endif
		}
	}
	else 
#endif
	{
		#pragma ignore warning(perf)
		*DstPtr = V;
	}
}

inline varying FVector4d VectorGather(const uniform FVector4d *varying SrcPtr)
{
	varying FVector4d Result;
	
#if USE_ISPC_OPTIMIZED_VECTORGATHER
	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
#if TARGET_WIDTH == 4
		AosToSoa4Explicit((const uniform double* varying)SrcPtr, (uniform double* uniform)&Result);
#else
		UniformLoad(SrcPtr, (uniform FVector4d* uniform)&Result);
		aos_to_soa4_ispc((uniform double* uniform)&Result, &Result.V[0], &Result.V[1], &Result.V[2], &Result.V[3]);
#endif
	}
	else
#endif
	{
		#pragma ignore warning(perf)
		Result = *SrcPtr;
	}

	return Result;
}

inline varying FVector4f VectorGather(const uniform FVector4f *varying SrcPtr)
{
	varying FVector4f Result;
	
#if USE_ISPC_OPTIMIZED_VECTORGATHER
	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
#if TARGET_WIDTH == 4
		AosToSoa4Explicit((const uniform float* varying)SrcPtr, (uniform float* uniform) & Result);
#else
		UniformLoad(SrcPtr, (uniform FVector4f* uniform)&Result);
		aos_to_soa4_ispc((uniform float* uniform)&Result, &Result.V[0], &Result.V[1], &Result.V[2], &Result.V[3]);
#endif
	}
	else
#endif
	{
		#pragma ignore warning(perf)
		Result = *SrcPtr;
	}

	return Result;
}

inline void VectorScatter(uniform FVector4d *varying DstPtr, const varying FVector4d &V)
{
#if USE_ISPC_OPTIMIZED_VECTORSCATTER
	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		unmasked
		{
#if TARGET_WIDTH == 4
			SoaToAos4Explicit((uniform double* uniform)&V, (uniform double* varying)DstPtr);
#else
			uniform FVector4d Result[programCount];
			soa_to_aos4_ispc(V.V[0], V.V[1], V.V[2], V.V[3], (uniform double* uniform)&Result);
			UniformStore(Result, DstPtr);
#endif
		}
	}
	else
#endif
	{
		#pragma ignore warning(perf)
		*DstPtr = V;
	}
}

inline void VectorScatter(uniform FVector4f *varying DstPtr, const varying FVector4f &V)
{
#if USE_ISPC_OPTIMIZED_VECTORSCATTER
	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		unmasked
		{
#if TARGET_WIDTH == 4
			SoaToAos4Explicit((uniform float* uniform)&V, (uniform float* varying)DstPtr);
#else
			uniform FVector4f Result[programCount];
			soa_to_aos4_ispc(V.V[0], V.V[1], V.V[2], V.V[3], (uniform float* uniform)&Result);
			UniformStore(Result, DstPtr);
#endif
		}
	}
	else
#endif
	{
		#pragma ignore warning(perf)
		*DstPtr = V;
	}
}

inline varying FIntVector VectorGather(const uniform FIntVector *varying SrcPtr)
{
	varying FIntVector Result;
	
#if USE_ISPC_OPTIMIZED_VECTORGATHER
	if (((1 << TARGET_WIDTH) - 1 ^ lanemask()) == 0)
	{
#if TARGET_WIDTH == 4
		AosToSoa3Explicit((uniform int32* varying)SrcPtr, (uniform int32* uniform)&Result);
#else
		UniformLoad(SrcPtr, (uniform FIntVector* uniform)&Result);
		aos_to_soa3_ispc((uniform int32* uniform)&Result, &Result.V[0], &Result.V[1], &Result.V[2]);
#endif
	}
	else
#endif
	{
		#pragma ignore warning(perf)
		Result = *SrcPtr;
	}

	return Result;
}

inline FVector2f operator+(const FVector2f& A, const FVector2f& B)
{
	FVector2f Result;

	Result.V[0] = A.V[0] + B.V[0];
	Result.V[1] = A.V[1] + B.V[1];

	return Result;
}

inline FVector2f operator*(const FVector2f& A, const FVector2f& B)
{
	FVector2f Result;

	Result.V[0] = A.V[0] * B.V[0];
	Result.V[1] = A.V[1] * B.V[1];

	return Result;
}

inline FVector2f operator*(const FVector2f& A, const float F)
{
	FVector2f Result;

	Result.V[0] = A.V[0] * F;
	Result.V[1] = A.V[1] * F;

	return Result;
}

inline FVector4d operator+(const FVector4d &A, const FVector4d &B)
{
	FVector4d Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] + B.V[i];
	}

	return Result;
}

inline FVector4f operator+(const FVector4f &A, const FVector4f &B)
{
	FVector4f Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] + B.V[i];
	}

	return Result;
}

inline FVector4f operator+(const FVector4f &A, const uniform FVector4f &B)
{
	FVector4f Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] + B.V[i];
	}

	return Result;
}

inline uniform FVector4d operator+(const uniform FVector4d &A, const uniform FVector4d &B)
{
	varying double S0, S1, Result;
	*((uniform FVector4d *uniform)&S0) = *((uniform FVector4d *uniform)&A);
	*((uniform FVector4d *uniform)&S1) = *((uniform FVector4d *uniform)&B);

	Result = S0 + S1;

	return *((uniform FVector4d *uniform)&Result);
}

inline uniform FVector4f operator+(const uniform FVector4f &A, const uniform FVector4f &B)
{
	varying float S0, S1, Result;
	*((uniform FVector4f *uniform)&S0) = *((uniform FVector4f *uniform)&A);
	*((uniform FVector4f *uniform)&S1) = *((uniform FVector4f *uniform)&B);

	Result = S0 + S1;

	return *((uniform FVector4f *uniform)&Result);
}

inline uniform FVector4d operator+(const uniform FVector4d &A, const uniform FVector4f &B)
{
	varying double S0, Result;
	varying float S1;
	*((uniform FVector4d *uniform)&S0) = *((uniform FVector4d *uniform)&A);
	*((uniform FVector4f *uniform)&S1) = *((uniform FVector4f *uniform)&B);

	Result = S0 + S1;

	return *((uniform FVector4d *uniform)&Result);
}

inline uniform FVector4d operator+(const uniform FVector4f &A, const uniform FVector4d &B)
{
	varying float S0;
	varying double S1, Result;
	*((uniform FVector4f *uniform)&S0) = *((uniform FVector4f *uniform)&A);
	*((uniform FVector4d *uniform)&S1) = *((uniform FVector4d *uniform)&B);

	Result = S0 + S1;

	return *((uniform FVector4d *uniform)&Result);
}

inline FVector4 operator+(const FVector4 &A, const uniform FVector4 &B)
{
	FVector4 Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] + B.V[i];
	}

	return Result;
}

inline FVector4 operator+(const FVector4 &A, const FReal F)
{
	FVector4 Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] + F;
	}

	return Result;
}

inline FVector4 operator+(const FReal F, const FVector4 &A)
{
	FVector4 Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = F + A.V[i];
	}

	return Result;
}

inline uniform FVector4 operator+(const uniform FVector4 &A, const uniform FReal F)
{
	const uniform FVector4 FVec = {{F,F,F,F}};
	return A + FVec;
}

inline FVector4d operator*(const FVector4d &A, const FVector4d &B)
{
	FVector4d Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] * B.V[i];
	}

	return Result;
}

inline FVector4f operator*(const FVector4f &A, const FVector4f &B)
{
	FVector4f Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] * B.V[i];
	}

	return Result;
}

inline FVector4f operator*(const uniform FVector4f &A, const FVector4f &B)
{
	FVector4f Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] * B.V[i];
	}

	return Result;
}

inline FVector4 operator*(const uniform FVector4 &A, const FVector4 &B)
{
	FVector4 Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] * B.V[i];
	}

	return Result;
}

inline FVector4d operator*(const FVector4d &A, const uniform FVector4d &B)
{
	FVector4d Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] * B.V[i];
	}

	return Result;
}

inline FVector4f operator*(const FVector4f &A, const uniform FVector4f &B)
{
	FVector4f Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] * B.V[i];
	}

	return Result;
}

inline uniform FVector4d operator*(const uniform FVector4d &A, const uniform FVector4d &B)
{
	varying double S0, S1, Result;
	*((uniform FVector4d *uniform)&S0) = *((uniform FVector4d *uniform)&A);
	*((uniform FVector4d *uniform)&S1) = *((uniform FVector4d *uniform)&B);

	Result = S0 * S1;

	return *((uniform FVector4d *uniform)&Result);
}

inline uniform FVector4f operator*(const uniform FVector4f &A, const uniform FVector4f &B)
{
	varying float S0, S1, Result;
	*((uniform FVector4f *uniform)&S0) = *((uniform FVector4f *uniform)&A);
	*((uniform FVector4f *uniform)&S1) = *((uniform FVector4f *uniform)&B);

	Result = S0 * S1;

	return *((uniform FVector4f *uniform)&Result);
}

inline FVector4d operator*(const FVector4d &A, const double F)
{
	FVector4d Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] * F;
	}

	return Result;
}

inline FVector4f operator*(const FVector4f &A, const float F)
{
	FVector4f Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] * F;
	}

	return Result;
}

inline FVector4 operator*(const FVector4 &A, const uniform FReal F)
{
	FVector4 Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] * F;
	}

	return Result;
}

inline FVector4d operator*(const uniform double F, const FVector4d &A)
{
	FVector4d Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = F * A.V[i];
	}

	return Result;
}

inline FVector4f operator*(const uniform float F, const FVector4f &A)
{
	FVector4f Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = F * A.V[i];
	}

	return Result;
}

inline uniform FVector4d operator*(const uniform FVector4d &A, const uniform double F)
{
	const uniform FVector4d FVec = {{F,F,F,F}};
	return A * FVec;
}

inline uniform FVector4f operator*(const uniform FVector4f &A, const uniform float F)
{
	const uniform FVector4f FVec = {{F,F,F,F}};
	return A * FVec;
}

inline uniform FVector4d operator*(const uniform double F, const uniform FVector4d &A)
{
	const uniform FVector4d FVec = {{F,F,F,F}};
	return FVec * A;
}

inline uniform FVector4f operator*(const uniform float F, const uniform FVector4f &A)
{
	const uniform FVector4f FVec = {{F,F,F,F}};
	return FVec * A;
}

inline FVector4d operator/(const FVector4d &A, const FVector4d &B)
{
	FVector4d Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] / B.V[i];
	}

	return Result;
}

inline FVector4f operator/(const FVector4f &A, const FVector4f &B)
{
	FVector4f Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] / B.V[i];
	}

	return Result;
}

inline FVector4 operator/(const FVector4 &A, const uniform FVector4 &B)
{
	FVector4 Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] / B.V[i];
	}

	return Result;
}

inline FVector4 operator/(const uniform FVector4 &A, const FVector4 &B)
{
	FVector4 Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] / B.V[i];
	}

	return Result;
}

inline uniform FVector4d operator/(const uniform FVector4d &A, const uniform FVector4d &B)
{
	varying double S0, S1, Result;
	*((uniform FVector4d *uniform)&S0) = *((uniform FVector4d *uniform)&A);
	*((uniform FVector4d *uniform)&S1) = *((uniform FVector4d *uniform)&B);

	Result = S0 / S1;

	return *((uniform FVector4d *uniform)&Result);
}

inline uniform FVector4f operator/(const uniform FVector4f &A, const uniform FVector4f &B)
{
	varying float S0, S1, Result;
	*((uniform FVector4f *uniform)&S0) = *((uniform FVector4f *uniform)&A);
	*((uniform FVector4f *uniform)&S1) = *((uniform FVector4f *uniform)&B);

	Result = S0 / S1;

	return *((uniform FVector4f *uniform)&Result);
}

inline uniform FVector4 operator/(const uniform FVector4 &A, const uniform FReal F)
{
	const uniform FVector4 FVec = {{F,F,F,F}};
	return A / FVec;
}

inline uniform FVector4 operator/(const uniform FReal F, const uniform FVector4 &A)
{
	const uniform FVector4 FVec = {{F,F,F,F}};
	return FVec / A;
}

inline FVector4d operator-(const FVector4d &A, const FVector4d &B)
{
	FVector4d Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] - B.V[i];
	}

	return Result;
}

inline FVector4f operator-(const FVector4f &A, const FVector4f &B)
{
	FVector4f Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] - B.V[i];
	}

	return Result;
}

inline uniform FVector4d operator-(const uniform FVector4d &A, const uniform FVector4d &B)
{
	varying double S0, S1, Result;
	*((uniform FVector4d *uniform)&S0) = *((uniform FVector4d *uniform)&A);
	*((uniform FVector4d *uniform)&S1) = *((uniform FVector4d *uniform)&B);

	Result = S0 - S1;

	return *((uniform FVector4d *uniform)&Result);
}

inline uniform FVector4f operator-(const uniform FVector4f &A, const uniform FVector4f &B)
{
	varying float S0, S1, Result;
	*((uniform FVector4f *uniform)&S0) = *((uniform FVector4f *uniform)&A);
	*((uniform FVector4f *uniform)&S1) = *((uniform FVector4f *uniform)&B);

	Result = S0 - S1;

	return *((uniform FVector4f *uniform)&Result);
}

inline FVector4 operator-(const FVector4 &A, const uniform FVector4 &B)
{
	FVector4 Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] - B.V[i];
	}

	return Result;
}

inline FVector4d operator-(const uniform FVector4d &A, const FVector4d &B)
{
	FVector4d Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] - B.V[i];
	}

	return Result;
}

inline FVector4f operator-(const uniform FVector4f &A, const FVector4f &B)
{
	FVector4f Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] - B.V[i];
	}

	return Result;
}

inline FVector4 operator-(const FVector4 &A, const FReal F)
{
	FVector4 Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] - F;
	}

	return Result;
}

inline FVector4 operator-(const FVector4 &A, const uniform FReal F)
{
	FVector4 Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = A.V[i] - F;
	}

	return Result;
}

inline FVector4d operator-(const uniform double F, const FVector4d &A)
{
	FVector4d Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = F - A.V[i];
	}

	return Result;
}

inline FVector4f operator-(const uniform float F, const FVector4f &A)
{
	FVector4f Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = F - A.V[i];
	}

	return Result;
}

inline uniform FVector4 operator-(const uniform FVector4 &A, const uniform FReal F)
{
	const uniform FVector4 FVec = {{F,F,F,F}};
	return A - FVec;
}

inline uniform FVector4 operator-(const uniform FReal F, const uniform FVector4 &A)
{
	const uniform FVector4 FVec = {{F,F,F,F}};
	return FVec - A;
}

inline FVector4d SetVector4(const double X, const double Y, const double Z, const double W)
{
	const FVector4d Result = {{X, Y, Z, W}};
	return Result;
}

inline FVector4f SetVector4(const float X, const float Y, const float Z, const float W)
{
	const FVector4f Result = {{X, Y, Z, W}};
	return Result;
}

inline uniform FVector4d SetVector4(const uniform double X, const uniform double Y, const uniform double Z, const uniform double W)
{
	const uniform FVector4d Result = {{X, Y, Z, W}};
	return Result;
}

inline uniform FVector4f SetVector4(const uniform float X, const uniform float Y, const uniform float Z, const uniform float W)
{
	const uniform FVector4f Result = {{X, Y, Z, W}};
	return Result;
}

inline FVector4d SetVector4(const FVector3d &V, const FReal W)
{
	const FVector4d Result = {{V.V[0], V.V[1], V.V[2], W}};
	return Result;
}

inline FVector4f SetVector4(const FVector3f &V, const FReal W)
{
	const FVector4f Result = {{V.V[0], V.V[1], V.V[2], W}};
	return Result;
}

inline uniform FVector4d SetVector4(const uniform FVector3d &V, const uniform double W)
{
	const uniform FVector4d Result = {{V.V[0], V.V[1], V.V[2], W}};
	return Result;
}

inline uniform FVector4f SetVector4(const uniform FVector3f &V, const uniform float W)
{
	const uniform FVector4f Result = {{V.V[0], V.V[1], V.V[2], W}};
	return Result;
}

inline uniform FVector4d SetVector4(const uniform double F)
{
	const uniform FVector4d Result = {{F, F, F, F}};
	return Result;
}

inline uniform FVector4f SetVector4(const uniform float F)
{
	const uniform FVector4f Result = {{F, F, F, F}};
	return Result;
}

inline varying FVector4d SetVector4(const varying double F)
{
	const varying FVector4d Result = {{F, F, F, F}};
	return Result;
}

inline varying FVector4f SetVector4(const varying float F)
{
	const varying FVector4f Result = {{F, F, F, F}};
	return Result;
}

inline FVector4d ConvertVector4fTo4d(const FVector4f &V)
{
	FVector4d Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = (varying double)V.V[i];
	}

	return Result;
}

inline FVector4f ConvertVector4dTo4f(const FVector4d &V)
{
	FVector4f Result;

	for(uniform int i = 0; i < 4; i++)
	{
		Result.V[i] = (varying float)V.V[i];
	}

	return Result;
}

inline uniform FVector4d ConvertVector4fTo4d(const uniform FVector4f &V)
{
	varying float S0;
	*((uniform FVector4f *uniform)&S0) = *((uniform FVector4f *uniform)&V);

	const varying double Result = (double)S0;

	return *((uniform FVector4d *uniform)&Result);
}

inline uniform FVector4f ConvertVector4dTo4f(const uniform FVector4d &V)
{
	varying double S0;
	*((uniform FVector4d *uniform)&S0) = *((uniform FVector4d *uniform)&V);

	const varying float Result = (float)S0;

	return *((uniform FVector4f *uniform)&Result);
}

inline FVector4 ConvertVector4dTo4Native(const FVector4d &V)
{
	return V;
}

inline FVector4 ConvertVector4fTo4Native(const FVector4f &V)
{
	return ConvertVector4fTo4d(V);
}

inline FVector4d ConvertVector4NativeTo4d(const FVector4 &V)
{
	return V;
}

inline FVector4f ConvertVector4NativeTo4f(const FVector4 &V)
{
	return ConvertVector4dTo4f(V);
}

inline uniform FVector4 ConvertVector4dTo4Native(const uniform FVector4d &V)
{
	return V;
}

inline uniform FVector4 ConvertVector4fTo4Native(const uniform FVector4f &V)
{
	return ConvertVector4fTo4d(V);
}

inline uniform FVector4d ConvertVector4NativeTo4d(const uniform FVector4 &V)
{
	return V;
}

inline uniform FVector4f ConvertVector4NativeTo4f(const uniform FVector4 &V)
{
	return ConvertVector4dTo4f(V);
}

inline FVector3d operator+(const FVector3d &A, const FVector3d &B)
{
	FVector3d Result;

	Result.V[0] = A.V[0] + B.V[0];
	Result.V[1] = A.V[1] + B.V[1];
	Result.V[2] = A.V[2] + B.V[2];

	return Result;
}

inline FVector3f operator+(const FVector3f &A, const FVector3f &B)
{
	FVector3f Result;

	Result.V[0] = A.V[0] + B.V[0];
	Result.V[1] = A.V[1] + B.V[1];
	Result.V[2] = A.V[2] + B.V[2];

	return Result;
}

inline uniform FVector3d operator+(const uniform FVector3d &A, const uniform FVector3d &B)
{
	uniform FVector4d S0, S1, Result;
	*((uniform FVector3d *uniform)&S0) = *((uniform FVector3d *uniform)&A);
	*((uniform FVector3d *uniform)&S1) = *((uniform FVector3d *uniform)&B);

	Result = S0 + S1;

	return *((uniform FVector3d *uniform)&Result);
}

inline uniform FVector3f operator+(const uniform FVector3f &A, const uniform FVector3f &B)
{
	uniform FVector4f S0, S1, Result;
	*((uniform FVector3f *uniform)&S0) = *((uniform FVector3f *uniform)&A);
	*((uniform FVector3f *uniform)&S1) = *((uniform FVector3f *uniform)&B);

	Result = S0 + S1;

	return *((uniform FVector3f *uniform)&Result);
}

inline FVector3d operator+(const FVector3d &A, const uniform FVector3d &B)
{
	FVector3d Result;

	Result.V[0] = A.V[0] + B.V[0];
	Result.V[1] = A.V[1] + B.V[1];
	Result.V[2] = A.V[2] + B.V[2];

	return Result;
}

inline FVector3f operator+(const FVector3f &A, const uniform FVector3f &B)
{
	FVector3f Result;

	Result.V[0] = A.V[0] + B.V[0];
	Result.V[1] = A.V[1] + B.V[1];
	Result.V[2] = A.V[2] + B.V[2];

	return Result;
}

inline FVector3d operator+(const uniform FVector3d &A, const FVector3d &B)
{
	FVector3d Result;

	Result.V[0] = A.V[0] + B.V[0];
	Result.V[1] = A.V[1] + B.V[1];
	Result.V[2] = A.V[2] + B.V[2];

	return Result;
}

inline FVector3f operator+(const uniform FVector3f &A, const FVector3f &B)
{
	FVector3f Result;

	Result.V[0] = A.V[0] + B.V[0];
	Result.V[1] = A.V[1] + B.V[1];
	Result.V[2] = A.V[2] + B.V[2];

	return Result;
}

inline FVector operator+(const FVector &A, const FReal F)
{
	FVector Result;

	Result.V[0] = A.V[0] + F;
	Result.V[1] = A.V[1] + F;
	Result.V[2] = A.V[2] + F;

	return Result;
}

inline uniform FVector operator+(const uniform FVector &A, const uniform FReal F)
{
	const uniform FVector FVec = {{F,F,F}};
	return A + FVec;
}

inline FVector3d operator*(const FVector3d &A, const FVector3d &B)
{
	FVector3d Result;

	Result.V[0] = A.V[0] * B.V[0];
	Result.V[1] = A.V[1] * B.V[1];
	Result.V[2] = A.V[2] * B.V[2];

	return Result;
}

inline FVector3f operator*(const FVector3f &A, const FVector3f &B)
{
	FVector3f Result;

	Result.V[0] = A.V[0] * B.V[0];
	Result.V[1] = A.V[1] * B.V[1];
	Result.V[2] = A.V[2] * B.V[2];

	return Result;
}

inline uniform FVector3d operator*(const uniform FVector3d &A, const uniform FVector3d &B)
{
	uniform FVector4d S0, S1, Result;
	*((uniform FVector3d *uniform)&S0) = *((uniform FVector3d *uniform)&A);
	*((uniform FVector3d *uniform)&S1) = *((uniform FVector3d *uniform)&B);

	Result = S0 * S1;

	return *((uniform FVector3d *uniform)&Result);
}

inline uniform FVector3f operator*(const uniform FVector3f &A, const uniform FVector3f &B)
{
	uniform FVector4f S0, S1, Result;
	*((uniform FVector3f *uniform)&S0) = *((uniform FVector3f *uniform)&A);
	*((uniform FVector3f *uniform)&S1) = *((uniform FVector3f *uniform)&B);

	Result = S0 * S1;

	return *((uniform FVector3f *uniform)&Result);
}

inline FVector3d operator*(const FVector3d &A, const uniform FVector3d &B)
{
	FVector3d Result;

	Result.V[0] = A.V[0] * B.V[0];
	Result.V[1] = A.V[1] * B.V[1];
	Result.V[2] = A.V[2] * B.V[2];

	return Result;
}

inline FVector3f operator*(const FVector3f &A, const uniform FVector3f &B)
{
	FVector3f Result;

	Result.V[0] = A.V[0] * B.V[0];
	Result.V[1] = A.V[1] * B.V[1];
	Result.V[2] = A.V[2] * B.V[2];

	return Result;
}

inline FVector3d operator*(const uniform FVector3d &A, const FVector3d &B)
{
	FVector3d Result;

	Result.V[0] = A.V[0] * B.V[0];
	Result.V[1] = A.V[1] * B.V[1];
	Result.V[2] = A.V[2] * B.V[2];

	return Result;
}

inline FVector3f operator*(const uniform FVector3f &A, const FVector3f &B)
{
	FVector3f Result;

	Result.V[0] = A.V[0] * B.V[0];
	Result.V[1] = A.V[1] * B.V[1];
	Result.V[2] = A.V[2] * B.V[2];

	return Result;
}

inline FVector operator*(const uniform FVector &A, const FIntVector &B)
{
	FVector Result;

	Result.V[0] = A.V[0] * B.V[0];
	Result.V[1] = A.V[1] * B.V[1];
	Result.V[2] = A.V[2] * B.V[2];

	return Result;
}

inline FVector3d operator*(const FVector3d &A, const double F)
{
	FVector3d Result;

	Result.V[0] = A.V[0] * F;
	Result.V[1] = A.V[1] * F;
	Result.V[2] = A.V[2] * F;

	return Result;
}

inline FVector3f operator*(const FVector3f &A, const float F)
{
	FVector3f Result;

	Result.V[0] = A.V[0] * F;
	Result.V[1] = A.V[1] * F;
	Result.V[2] = A.V[2] * F;

	return Result;
}

inline FVector3d operator*(const uniform FVector3d &A, const varying double F)
{
	FVector3d Result;

	Result.V[0] = A.V[0] * F;
	Result.V[1] = A.V[1] * F;
	Result.V[2] = A.V[2] * F;

	return Result;
}

inline FVector3f operator*(const uniform FVector3f &A, const varying float F)
{
	FVector3f Result;

	Result.V[0] = A.V[0] * F;
	Result.V[1] = A.V[1] * F;
	Result.V[2] = A.V[2] * F;

	return Result;
}

inline FVector3d operator*(const FVector3d &A, const uniform double F)
{
	FVector3d Result;

	Result.V[0] = A.V[0] * F;
	Result.V[1] = A.V[1] * F;
	Result.V[2] = A.V[2] * F;

	return Result;
}

inline FVector3f operator*(const FVector3f &A, const uniform float F)
{
	FVector3f Result;

	Result.V[0] = A.V[0] * F;
	Result.V[1] = A.V[1] * F;
	Result.V[2] = A.V[2] * F;

	return Result;
}

inline FVector3d operator*(const double F, const FVector3d &A)
{
	FVector3d Result;

	Result.V[0] = F * A.V[0];
	Result.V[1] = F * A.V[1];
	Result.V[2] = F * A.V[2];

	return Result;
}

inline FVector3f operator*(const float F, const FVector3f &A)
{
	FVector3f Result;

	Result.V[0] = F * A.V[0];
	Result.V[1] = F * A.V[1];
	Result.V[2] = F * A.V[2];

	return Result;
}

inline uniform FVector3d operator*(const uniform FVector3d &A, const uniform double F)
{
	const uniform FVector3d FVec = {{F,F,F}};
	return A * FVec;
}

inline uniform FVector3f operator*(const uniform FVector3f &A, const uniform float F)
{
	const uniform FVector3f FVec = {{F,F,F}};
	return A * FVec;
}

inline uniform FVector operator*(const uniform FReal F, const uniform FVector &A)
{
	const uniform FVector FVec = {{F,F,F}};
	return FVec * A;
}

inline FVector3d operator-(const FVector3d &A, const FVector3d &B)
{
	FVector3d Result;

	Result.V[0] = A.V[0] - B.V[0];
	Result.V[1] = A.V[1] - B.V[1];
	Result.V[2] = A.V[2] - B.V[2];

	return Result;
}

inline FVector3f operator-(const FVector3f &A, const FVector3f &B)
{
	FVector3f Result;

	Result.V[0] = A.V[0] - B.V[0];
	Result.V[1] = A.V[1] - B.V[1];
	Result.V[2] = A.V[2] - B.V[2];

	return Result;
}

inline uniform FVector3d operator-(const uniform FVector3d &A, const uniform FVector3d &B)
{
	uniform FVector4d S0, S1, Result;
	*((uniform FVector3d *uniform)&S0) = *((uniform FVector3d *uniform)&A);
	*((uniform FVector3d *uniform)&S1) = *((uniform FVector3d *uniform)&B);

	Result = S0 - S1;

	return *((uniform FVector3d *uniform)&Result);
}

inline uniform FVector3f operator-(const uniform FVector3f &A, const uniform FVector3f &B)
{
	uniform FVector4f S0, S1, Result;
	*((uniform FVector3f *uniform)&S0) = *((uniform FVector3f *uniform)&A);
	*((uniform FVector3f *uniform)&S1) = *((uniform FVector3f *uniform)&B);

	Result = S0 - S1;

	return *((uniform FVector3f *uniform)&Result);
}

inline FVector3d operator-(const FVector3d &A, const uniform FVector3d &B)
{
	FVector3d Result;

	Result.V[0] = A.V[0] - B.V[0];
	Result.V[1] = A.V[1] - B.V[1];
	Result.V[2] = A.V[2] - B.V[2];

	return Result;
}

inline FVector3f operator-(const FVector3f &A, const uniform FVector3f &B)
{
	FVector3f Result;

	Result.V[0] = A.V[0] - B.V[0];
	Result.V[1] = A.V[1] - B.V[1];
	Result.V[2] = A.V[2] - B.V[2];

	return Result;
}

inline FVector3d operator-(const uniform FVector3d &A, const FVector3d &B)
{
	FVector3d Result;

	Result.V[0] = A.V[0] - B.V[0];
	Result.V[1] = A.V[1] - B.V[1];
	Result.V[2] = A.V[2] - B.V[2];

	return Result;
}

inline FVector3f operator-(const uniform FVector3f &A, const FVector3f &B)
{
	FVector3f Result;

	Result.V[0] = A.V[0] - B.V[0];
	Result.V[1] = A.V[1] - B.V[1];
	Result.V[2] = A.V[2] - B.V[2];

	return Result;
}

inline FVector operator-(const FVector &A, const FReal F)
{
	FVector Result;

	Result.V[0] = A.V[0] - F;
	Result.V[1] = A.V[1] - F;
	Result.V[2] = A.V[2] - F;

	return Result;
}

inline uniform FVector operator-(const uniform FVector &A, const uniform FReal F)
{
	const uniform FVector FVec = {{F,F,F}};
	return A - FVec;
}

inline uniform FVector operator-(const uniform FReal F, const uniform FVector &A)
{
	const uniform FVector FVec = {{F,F,F}};
	return FVec - A;
}

inline FVector operator-(const FReal F, const FVector &A)
{
	FVector Result;

	Result.V[0] = F - A.V[0];
	Result.V[1] = F - A.V[1];
	Result.V[2] = F - A.V[2];

	return Result;
}

inline FVector3d operator/(const FVector3d &A, const FVector3d &B)
{
	FVector3d Result;

	Result.V[0] = A.V[0] / B.V[0];
	Result.V[1] = A.V[1] / B.V[1];
	Result.V[2] = A.V[2] / B.V[2];

	return Result;
}

inline FVector3f operator/(const FVector3f &A, const FVector3f &B)
{
	FVector3f Result;

	Result.V[0] = A.V[0] / B.V[0];
	Result.V[1] = A.V[1] / B.V[1];
	Result.V[2] = A.V[2] / B.V[2];

	return Result;
}

inline uniform FVector3d operator/(const uniform FVector3d &A, const uniform FVector3d &B)
{
	uniform FVector4d S0, S1, Result;
	*((uniform FVector3d *uniform)&S0) = *((uniform FVector3d *uniform)&A);
	*((uniform FVector3d *uniform)&S1) = *((uniform FVector3d *uniform)&B);

	Result = S0 / S1;

	return *((uniform FVector3d *uniform)&Result);
}

inline uniform FVector3f operator/(const uniform FVector3f &A, const uniform FVector3f &B)
{
	uniform FVector4f S0, S1, Result;
	*((uniform FVector3f *uniform)&S0) = *((uniform FVector3f *uniform)&A);
	*((uniform FVector3f *uniform)&S1) = *((uniform FVector3f *uniform)&B);

	Result = S0 / S1;

	return *((uniform FVector3f *uniform)&Result);
}

inline FVector operator/(const FVector &A, const uniform FVector &B)
{
	FVector Result;

	Result.V[0] = A.V[0] / B.V[0];
	Result.V[1] = A.V[1] / B.V[1];
	Result.V[2] = A.V[2] / B.V[2];

	return Result;
}

inline FVector3d operator/(const FVector3d &A, const double F)
{
	FVector3d Result;

	Result.V[0] = A.V[0] / F;
	Result.V[1] = A.V[1] / F;
	Result.V[2] = A.V[2] / F;

	return Result;
}

inline FVector3f operator/(const FVector3f &A, const float F)
{
	FVector3f Result;

	Result.V[0] = A.V[0] / F;
	Result.V[1] = A.V[1] / F;
	Result.V[2] = A.V[2] / F;

	return Result;
}

inline uniform FVector3d operator/(const uniform FVector3d &A, const uniform double F)
{
	const uniform FVector3d FVec = {{F,F,F}};
	return A / FVec;
}

inline uniform FVector3f operator/(const uniform FVector3f &A, const uniform float F)
{
	const uniform FVector3f FVec = {{F,F,F}};
	return A / FVec;
}

inline uniform FVector operator/(const uniform FReal F, const uniform FVector &A)
{
	const uniform FVector FVec = {{F,F,F}};
	return FVec / A;
}

inline FVector3d SetVector(const double X, const double Y, const double Z)
{
	const FVector3d Result = {{X, Y, Z}};
	return Result;
}

inline FVector3f SetVector(const float X, const float Y, const float Z)
{
	const FVector3f Result = {{X, Y, Z}};
	return Result;
}

inline uniform FVector3d SetVector(const uniform double X, const uniform double Y, const uniform double Z)
{
	const uniform FVector3d Result = {{X, Y, Z}};
	return Result;
}

inline uniform FVector3f SetVector(const uniform float X, const uniform float Y, const uniform float Z)
{
	const uniform FVector3f Result = {{X, Y, Z}};
	return Result;
}

inline uniform FVector SetVector(const uniform FReal F)
{
	const uniform FVector Result = {{F, F, F}};
	return Result;
}

inline FVector SetVector(const FReal F)
{
	const FVector Result = {{F, F, F}};
	return Result;
}

inline FVector3d SetVector(const FVector4d &Vec)
{
	const FVector3d Result = {{Vec.V[0], Vec.V[1], Vec.V[2]}};
	return Result;
}

inline FVector3f SetVector(const FVector4f &Vec)
{
	const FVector3f Result = {{Vec.V[0], Vec.V[1], Vec.V[2]}};
	return Result;
}

inline uniform FVector3f SetVector(const uniform FVector4f &Vec)
{
	const uniform FVector3f Result = {{Vec.V[0], Vec.V[1], Vec.V[2]}};
	return Result;
}

inline uniform FVector3d SetVector(const uniform FVector4d &Vec)
{
	const uniform FVector3d Result = {{Vec.V[0], Vec.V[1], Vec.V[2]}};
	return Result;
}

inline FVector3d ConvertVector3fTo3d(const FVector3f &V)
{
	FVector3d Result;

	for(uniform int i = 0; i < 3; i++)
	{
		Result.V[i] = (varying double)V.V[i];
	}

	return Result;
}

inline FVector3f ConvertVector3dTo3f(const FVector3d &V)
{
	FVector3f Result;

	for(uniform int i = 0; i < 3; i++)
	{
		Result.V[i] = (varying float)V.V[i];
	}

	return Result;
}

inline uniform FVector3d ConvertVector3fTo3d(const uniform FVector3f &V)
{
	varying float S0;
	*((uniform FVector3f *uniform)&S0) = *((uniform FVector3f *uniform)&V);

	const varying double Result = (double)S0;

	return *((uniform FVector3d *uniform)&Result);
}

inline uniform FVector3f ConvertVector3dTo3f(const uniform FVector3d &V)
{
	varying double S0;
	*((uniform FVector3d *uniform)&S0) = *((uniform FVector3d *uniform)&V);

	const varying float Result = (float)S0;

	return *((uniform FVector3f *uniform)&Result);
}

inline FVector ConvertVector3dTo3Native(const FVector3d &V)
{
	return V;
}

inline FVector ConvertVector3fTo3Native(const FVector3f &V)
{
	return ConvertVector3fTo3d(V);
}

inline FVector3d ConvertVector3NativeTo3d(const FVector &V)
{
	return V;
}

inline FVector3f ConvertVector3NativeTo3f(const FVector &V)
{
	return ConvertVector3dTo3f(V);
}

inline uniform FVector ConvertVector3dTo3Native(const uniform FVector3d &V)
{
	return V;
}

inline uniform FVector ConvertVector3fTo3Native(const uniform FVector3f &V)
{
	return ConvertVector3fTo3d(V);
}

inline uniform FVector3d ConvertVector3NativeTo3d(const uniform FVector &V)
{
	return V;
}

inline uniform FVector3f ConvertVector3NativeTo3f(const uniform FVector &V)
{
	return ConvertVector3dTo3f(V);
}

inline uniform FVector8 operator+(const uniform FVector8 &A, const uniform FVector8 &B)
{
	varying float S0, S1, Result;
	*((uniform FVector8 *uniform)&S0) = *((uniform FVector8 *uniform)&A);
	*((uniform FVector8 *uniform)&S1) = *((uniform FVector8 *uniform)&B);

	Result = S0 + S1;

	return *((uniform FVector8 *uniform)&Result);
}

inline uniform FVector8 operator-(const uniform FVector8 &A, const uniform FVector8 &B)
{
	varying float S0, S1, Result;
	*((uniform FVector8 *uniform)&S0) = *((uniform FVector8 *uniform)&A);
	*((uniform FVector8 *uniform)&S1) = *((uniform FVector8 *uniform)&B);

	Result = S0 - S1;

	return *((uniform FVector8 *uniform)&Result);
}

inline uniform FVector8 operator*(const uniform FVector8 &A, const uniform FVector8 &B)
{
	varying float S0, S1, Result;
	*((uniform FVector8 *uniform)&S0) = *((uniform FVector8 *uniform)&A);
	*((uniform FVector8 *uniform)&S1) = *((uniform FVector8 *uniform)&B);

	Result = S0 * S1;

	return *((uniform FVector8 *uniform)&Result);
}

inline uniform FVector8 SetVector8(const uniform float X0, const uniform float Y0, const uniform float Z0, const uniform float W0,
									const uniform float X1, const uniform float Y1, const uniform float Z1, const uniform float W1)
{
	const uniform FVector8 Result = {{X0, Y0, Z0, W0, X1, Y1, Z1, W1}};
	return Result;
}

inline uniform FVector8 SetVector8(const uniform FVector4 &V0, const uniform FVector4 &V1)
{
	const uniform FVector8 Result = {{V0.V[0], V0.V[1], V0.V[2], V0.V[3], V1.V[0], V1.V[1], V1.V[2], V1.V[3]}};
	return Result;
}

inline uniform FVector8 SetVector8(const uniform FVector4 &V)
{
	const uniform FVector8 Result = {{V.V[0], V.V[1], V.V[2], V.V[3], V.V[0], V.V[1], V.V[2], V.V[3]}};
	return Result;
}

inline FIntVector operator+(const FIntVector &A, const FIntVector &B)
{
	FIntVector Result;

	Result.V[0] = A.V[0] + B.V[0];
	Result.V[1] = A.V[1] + B.V[1];
	Result.V[2] = A.V[2] + B.V[2];

	return Result;
}

inline uniform FIntVector operator+(const uniform FIntVector &A, const uniform FIntVector &B)
{
	uniform FIntVector Result;

	Result.V[0] = A.V[0] + B.V[0];
	Result.V[1] = A.V[1] + B.V[1];
	Result.V[2] = A.V[2] + B.V[2];

	return Result;
}

inline FIntVector operator+(const FIntVector &A, const uniform FIntVector &B)
{
	FIntVector Result;

	Result.V[0] = A.V[0] + B.V[0];
	Result.V[1] = A.V[1] + B.V[1];
	Result.V[2] = A.V[2] + B.V[2];

	return Result;
}

inline FIntVector operator+(const FIntVector &A, const int F)
{
	FIntVector Result;

	Result.V[0] = A.V[0] + F;
	Result.V[1] = A.V[1] + F;
	Result.V[2] = A.V[2] + F;

	return Result;
}

inline uniform FIntVector operator+(const uniform FIntVector &A, const uniform float F)
{
	uniform FIntVector Result;

	Result.V[0] = A.V[0] + F;
	Result.V[1] = A.V[1] + F;
	Result.V[2] = A.V[2] + F;

	return Result;
}

inline FIntVector operator*(const FIntVector &A, const FIntVector &B)
{
	FIntVector Result;

	Result.V[0] = A.V[0] * B.V[0];
	Result.V[1] = A.V[1] * B.V[1];
	Result.V[2] = A.V[2] * B.V[2];

	return Result;
}

inline uniform FIntVector operator*(const uniform FIntVector &A, const uniform FIntVector &B)
{
	uniform FIntVector Result;

	Result.V[0] = A.V[0] * B.V[0];
	Result.V[1] = A.V[1] * B.V[1];
	Result.V[2] = A.V[2] * B.V[2];

	return Result;
}

inline FIntVector operator*(const FIntVector &A, const uniform FIntVector &B)
{
	FIntVector Result;

	Result.V[0] = A.V[0] * B.V[0];
	Result.V[1] = A.V[1] * B.V[1];
	Result.V[2] = A.V[2] * B.V[2];

	return Result;
}

inline FIntVector operator*(const FIntVector &A, const int F)
{
	FIntVector Result;

	Result.V[0] = A.V[0] * F;
	Result.V[1] = A.V[1] * F;
	Result.V[2] = A.V[2] * F;

	return Result;
}

inline uniform FIntVector operator*(const uniform FIntVector &A, const uniform int F)
{
	uniform FIntVector Result;

	Result.V[0] = A.V[0] * F;
	Result.V[1] = A.V[1] * F;
	Result.V[2] = A.V[2] * F;

	return Result;
}

inline FIntVector operator-(const FIntVector &A, const FIntVector &B)
{
	FIntVector Result;

	Result.V[0] = A.V[0] - B.V[0];
	Result.V[1] = A.V[1] - B.V[1];
	Result.V[2] = A.V[2] - B.V[2];

	return Result;
}

inline uniform FIntVector operator-(const uniform FIntVector &A, const uniform FIntVector &B)
{
	uniform FIntVector Result;

	Result.V[0] = A.V[0] - B.V[0];
	Result.V[1] = A.V[1] - B.V[1];
	Result.V[2] = A.V[2] - B.V[2];

	return Result;
}

inline FIntVector operator-(const FIntVector &A, const uniform FIntVector &B)
{
	FIntVector Result;

	Result.V[0] = A.V[0] - B.V[0];
	Result.V[1] = A.V[1] - B.V[1];
	Result.V[2] = A.V[2] - B.V[2];

	return Result;
}

inline FIntVector operator-(const uniform FIntVector &A, const FIntVector &B)
{
	FIntVector Result;

	Result.V[0] = A.V[0] - B.V[0];
	Result.V[1] = A.V[1] - B.V[1];
	Result.V[2] = A.V[2] - B.V[2];

	return Result;
}

inline FIntVector operator-(const FIntVector &A, const int F)
{
	FIntVector Result;

	Result.V[0] = A.V[0] - F;
	Result.V[1] = A.V[1] - F;
	Result.V[2] = A.V[2] - F;

	return Result;
}

inline FIntVector operator-(const int F, const FIntVector &A)
{
	FIntVector Result;

	Result.V[0] = F - A.V[0];
	Result.V[1] = F - A.V[1];
	Result.V[2] = F - A.V[2];

	return Result;
}

inline FIntVector SetIntVector(const int X, const int Y, const int Z)
{
	FIntVector Result;

	Result.V[0] = X;
	Result.V[1] = Y;
	Result.V[2] = Z;

	return Result;
}

inline uniform FIntVector SetIntVector(const uniform int X, const uniform int Y, const uniform int Z)
{
	uniform FIntVector Result;

	Result.V[0] = X;
	Result.V[1] = Y;
	Result.V[2] = Z;

	return Result;
}

inline FIntVector SetIntVector(const FVector &A)
{
	FIntVector Result;

	Result.V[0] = (int)A.V[0];
	Result.V[1] = (int)A.V[1];
	Result.V[2] = (int)A.V[2];

	return Result;
}

inline uniform bool VectorIsAnyNearlyZero(const uniform FVector &V, const uniform FReal Tolerance)
{
	varying FReal S0 = REAL_MAX;

	*((uniform FVector *uniform)&S0) = *((uniform FVector *uniform)&V);

	return any(abs(S0) <= Tolerance);
}

inline uniform bool VectorIsAllNearlyZero(const uniform FVector3d &V, const uniform double Tolerance)
{
	varying double S0 = DBL_MIN;

	*((uniform FVector3d *uniform)&S0) = *((uniform FVector3d *uniform)&V);

	return all(abs(S0) <= Tolerance);
}

inline uniform bool VectorIsAllNearlyZero(const uniform FVector3f &V, const uniform float Tolerance)
{
	varying float S0 = FLT_MIN;

	*((uniform FVector3f *uniform)&S0) = *((uniform FVector3f *uniform)&V);

	return all(abs(S0) <= Tolerance);
}

template<typename T, typename V>
inline T VectorLerp(const T& A, const T& B, const V Alpha)
{
	return (A + (B - A) * Alpha);
}

inline FVector VectorFloor(const FVector &A)
{
	return SetVector(floor(A.V[0]), floor(A.V[1]), floor(A.V[2]));
}

template<typename T, typename V>
inline T VectorClamp(const T &A, const V &B, const V &C)
{
	return SetVector(clamp(A.V[0], B.V[0], C.V[0]),
		clamp(A.V[1], B.V[1], C.V[1]),
		clamp(A.V[2], B.V[2], C.V[2]));
}

template <typename T>
inline T VectorAdd(const T& A, const T& B)
{
	return A + B;
}

template <typename T>
inline T VectorSubtract(const T& A, const T& B)
{
	return A - B;
}

inline FVector4d VectorMultiply(const FVector4d &A, const FVector4d &B)
{
	return A * B;
}

inline FVector4f VectorMultiply(const FVector4f &A, const FVector4f &B)
{
	return A * B;
}

inline FVector4 VectorMultiply(const FVector4 &A, const uniform FVector4 &B)
{
	return A * B;
}

inline FVector4 VectorMultiply(const uniform FVector4 &A, const FVector4 &B)
{
	return A * B;
}

inline FVector4f VectorMultiply(const uniform FVector4f &A, const FVector4f &B)
{
	return A * B;
}

inline uniform FVector4d VectorMultiply(const uniform FVector4d &A, const uniform FVector4d &B)
{
	return A * B;
}

inline uniform FVector4f VectorMultiply(const uniform FVector4f &A, const uniform FVector4f &B)
{
	return A * B;
}

inline uniform FVector4 VectorMultiply(const uniform FVector4 &A, const uniform FReal F)
{
	return A * F;
}

inline uniform FVector8 VectorMultiply(const uniform FVector8 &A, const uniform FVector8 &B)
{
	return A * B;
}

template <typename T>
inline T VectorDivide(const T& A, const T& B)
{
	return A / B;
}

inline FVector4d VectorMultiplyAdd(const FVector4d &A, const FVector4d &B, const FVector4d &C)
{
	FVector4d Result;

	Result.V[0] = A.V[0] * B.V[0] + C.V[0];
	Result.V[1] = A.V[1] * B.V[1] + C.V[1];
	Result.V[2] = A.V[2] * B.V[2] + C.V[2];
	Result.V[3] = A.V[3] * B.V[3] + C.V[3];

	return Result;
}

inline FVector4f VectorMultiplyAdd(const FVector4f &A, const FVector4f &B, const FVector4f &C)
{
	FVector4f Result;

	Result.V[0] = A.V[0] * B.V[0] + C.V[0];
	Result.V[1] = A.V[1] * B.V[1] + C.V[1];
	Result.V[2] = A.V[2] * B.V[2] + C.V[2];
	Result.V[3] = A.V[3] * B.V[3] + C.V[3];

	return Result;
}

inline FVector4 VectorMultiplyAdd(const FVector4& A, const uniform FVector4 &B, const FVector4 &C)
{
	FVector4 Result;

	Result.V[0] = A.V[0] * B.V[0] + C.V[0];
	Result.V[1] = A.V[1] * B.V[1] + C.V[1];
	Result.V[2] = A.V[2] * B.V[2] + C.V[2];
	Result.V[3] = A.V[3] * B.V[3] + C.V[3];

	return Result;
}

inline FVector4 VectorMultiplyAdd(const uniform FVector4& A, const FVector4 &B, const FVector4 &C)
{
	FVector4 Result;

	Result.V[0] = A.V[0] * B.V[0] + C.V[0];
	Result.V[1] = A.V[1] * B.V[1] + C.V[1];
	Result.V[2] = A.V[2] * B.V[2] + C.V[2];
	Result.V[3] = A.V[3] * B.V[3] + C.V[3];

	return Result;
}

inline FVector4f VectorMultiplyAdd(const uniform FVector4f& A, const FVector4f &B, const FVector4f &C)
{
	FVector4f Result;

	Result.V[0] = A.V[0] * B.V[0] + C.V[0];
	Result.V[1] = A.V[1] * B.V[1] + C.V[1];
	Result.V[2] = A.V[2] * B.V[2] + C.V[2];
	Result.V[3] = A.V[3] * B.V[3] + C.V[3];

	return Result;
}

inline FVector4 VectorMultiplyAdd(const FReal F, const FVector4 &B, const FVector4 &C)
{
	FVector4 Result;

	Result.V[0] = F * B.V[0] + C.V[0];
	Result.V[1] = F * B.V[1] + C.V[1];
	Result.V[2] = F * B.V[2] + C.V[2];
	Result.V[3] = F * B.V[3] + C.V[3];

	return Result;
}

inline FVector4f VectorMultiplyAdd(const float F, const FVector4f& B, const FVector4f& C)
{
	FVector4f Result;

	Result.V[0] = F * B.V[0] + C.V[0];
	Result.V[1] = F * B.V[1] + C.V[1];
	Result.V[2] = F * B.V[2] + C.V[2];
	Result.V[3] = F * B.V[3] + C.V[3];

	return Result;
}

inline uniform FVector4d VectorMultiplyAdd(const uniform FVector4d &A, const uniform FVector4d &B, const uniform FVector4d &C)
{
	varying double S0, S1, S2, Result;
	*((uniform FVector4d *uniform)&S0) = *((uniform FVector4d *uniform)&A);
	*((uniform FVector4d *uniform)&S1) = *((uniform FVector4d *uniform)&B);
	*((uniform FVector4d *uniform)&S2) = *((uniform FVector4d *uniform)&C);

	Result = S0 * S1 + S2;

	return *((uniform FVector4d *uniform)&Result);
}

inline uniform FVector4f VectorMultiplyAdd(const uniform FVector4f &A, const uniform FVector4f &B, const uniform FVector4f &C)
{
	varying float S0, S1, S2, Result;
	*((uniform FVector4f *uniform)&S0) = *((uniform FVector4f *uniform)&A);
	*((uniform FVector4f *uniform)&S1) = *((uniform FVector4f *uniform)&B);
	*((uniform FVector4f *uniform)&S2) = *((uniform FVector4f *uniform)&C);

	Result = S0 * S1 + S2;

	return *((uniform FVector4f *uniform)&Result);
}

inline FVector4 VectorMultiplyAdd(const FVector4& A, const uniform FReal B, const FVector4 &C)
{
	FVector4 Result;

	Result.V[0] = A.V[0] * B + C.V[0];
	Result.V[1] = A.V[1] * B + C.V[1];
	Result.V[2] = A.V[2] * B + C.V[2];
	Result.V[3] = A.V[3] * B + C.V[3];

	return Result;
}

inline uniform FVector4 VectorMultiplyAdd(const uniform FVector4& A, const uniform FReal S1, const uniform FVector4 &C)
{
	varying FReal S0, S2, Result;
	*((uniform FVector4 *uniform)&S0) = *((uniform FVector4 *uniform)&A);
	*((uniform FVector4 *uniform)&S2) = *((uniform FVector4 *uniform)&C);

	Result = S0 * S1 + S2;

	return *((uniform FVector4 *uniform)&Result);
}

inline uniform FVector4 VectorMultiplyAdd(const uniform FReal S0, const uniform FVector4 &B, const uniform FVector4 &C)
{
	varying FReal S1, S2, Result;
	*((uniform FVector4 *uniform)&S1) = *((uniform FVector4 *uniform)&B);
	*((uniform FVector4 *uniform)&S2) = *((uniform FVector4 *uniform)&C);

	Result = S0 * S1 + S2;

	return *((uniform FVector4 *uniform)&Result);
}

inline uniform FVector8 VectorMultiplyAdd(const uniform FVector8& A, const uniform FVector8 &B, const uniform FVector8 &C)
{
	varying float S0, S1, S2, Result;
	*((uniform FVector8 *uniform)&S0) = *((uniform FVector8 *uniform)&A);
	*((uniform FVector8 *uniform)&S1) = *((uniform FVector8 *uniform)&B);
	*((uniform FVector8 *uniform)&S2) = *((uniform FVector8 *uniform)&C);

	Result = S0 * S1 + S2;

	return *((uniform FVector8 *uniform)&Result);
}

inline FVector4 VectorAbs(const FVector4& A)
{
	FVector4 Result;

	Result.V[0] = abs(A.V[0]);
	Result.V[1] = abs(A.V[1]);
	Result.V[2] = abs(A.V[2]);
	Result.V[3] = abs(A.V[3]);

	return Result;
}

inline uniform FVector4 VectorAbs(const uniform FVector4& A)
{
	varying FReal S0, Result;
	*((uniform FVector4 *uniform)&S0) = *((uniform FVector4 *uniform)&A);

	Result = abs(S0);

	return *((uniform FVector4 *uniform)&Result);
}

inline uniform FVector VectorAbs(const uniform FVector& A)
{
	uniform FVector4 S0, Result;
	*((uniform FVector *uniform)&S0) = *((uniform FVector *uniform)&A);

	Result = VectorAbs(S0);

	return *((uniform FVector *uniform)&Result);
}

inline bool IsVectorEqual(const FVector3d &A, const FVector3d &B)
{
	return A.V[0] == B.V[0] && A.V[1] == B.V[1] && A.V[2] == B.V[2];
}

inline bool IsVectorEqual(const FVector3f &A, const FVector3f &B)
{
	return A.V[0] == B.V[0] && A.V[1] == B.V[1] && A.V[2] == B.V[2];
}

inline bool IsVectorLessEqual(const FVector4 &A, const uniform FVector4 &B)
{
	return A.V[0] <= B.V[0] && A.V[1] <= B.V[1] && A.V[2] <= B.V[2] && A.V[3] <= B.V[3];
}

inline bool IsVectorLessEqual(const FVector &A, const uniform FVector &B)
{
	return A.V[0] <= B.V[0] && A.V[1] <= B.V[1] && A.V[2] <= B.V[2];
}

inline uniform bool IsVectorGreaterEqual(const uniform FVector4 &A, const uniform FVector4 &B)
{
	varying bool b = true;
	varying FReal S0, S1;
	*((uniform FVector4 *uniform)&S0) = *((uniform FVector4 *uniform)&A);
	*((uniform FVector4 *uniform)&S1) = *((uniform FVector4 *uniform)&B);

	b = S0 >= S1 ? true : false;

	return all(b);
}

inline bool IsVectorGreaterEqual(const FVector &A, const uniform FVector &B)
{
	return A.V[0] >= B.V[0] && A.V[1] >= B.V[1] && A.V[2] >= B.V[2];
}

inline FVector4d VectorCompareGE(const FVector4d &A, const uniform FVector4d &B)
{
	FVector4d Result;

	Result.V[0] = select(A.V[0] >= B.V[0], doublebits(DOUBLE_NAN), doublebits(0));
	Result.V[1] = select(A.V[1] >= B.V[1], doublebits(DOUBLE_NAN), doublebits(0));
	Result.V[2] = select(A.V[2] >= B.V[2], doublebits(DOUBLE_NAN), doublebits(0));
	Result.V[3] = select(A.V[3] >= B.V[3], doublebits(DOUBLE_NAN), doublebits(0));

	return Result;
}

inline FVector4f VectorCompareGE(const FVector4f &A, const uniform FVector4f &B)
{
	FVector4f Result;

	Result.V[0] = select(A.V[0] >= B.V[0], floatbits(FLOAT_NAN), floatbits(0));
	Result.V[1] = select(A.V[1] >= B.V[1], floatbits(FLOAT_NAN), floatbits(0));
	Result.V[2] = select(A.V[2] >= B.V[2], floatbits(FLOAT_NAN), floatbits(0));
	Result.V[3] = select(A.V[3] >= B.V[3], floatbits(FLOAT_NAN), floatbits(0));

	return Result;
}

inline FVector4 VectorCompareGE(const uniform FVector4 &A, const FVector4 &B)
{
	FVector4 Result;

	Result.V[0] = select(A.V[0] >= B.V[0], REAL_BITS(NAN), REAL_BITS(0));
	Result.V[1] = select(A.V[1] >= B.V[1], REAL_BITS(NAN), REAL_BITS(0));
	Result.V[2] = select(A.V[2] >= B.V[2], REAL_BITS(NAN), REAL_BITS(0));
	Result.V[3] = select(A.V[3] >= B.V[3], REAL_BITS(NAN), REAL_BITS(0));

	return Result;
}

inline uniform FVector4 VectorCompareGE(const uniform FVector4 &A, const uniform FVector4 &B)
{
	varying FReal S0, S1, Result;
	*((uniform FVector4 *uniform)&S0) = *((uniform FVector4 *uniform)&A);
	*((uniform FVector4 *uniform)&S1) = *((uniform FVector4 *uniform)&B);

	Result = S0 >= S1 ? REAL_BITS(NAN) : REAL_BITS(0);

	return *((uniform FVector4 *uniform)&Result);
}

inline uniform FVector4 VectorCompareGE(const uniform FReal F, const uniform FVector4 &B)
{
	varying FReal S1, Result;
	*((uniform FVector4 *uniform)&S1) = *((uniform FVector4 *uniform)&B);

	Result = F >= S1 ? REAL_BITS(NAN) : REAL_BITS(0);

	return *((uniform FVector4 *uniform)&Result);
}

inline uniform FVector VectorCompareGE(const uniform FVector& A, const uniform FVector& B)
{
	uniform FVector4 S0, S1, Result;
	*((uniform FVector *uniform)&S0) = *((uniform FVector *uniform)&A);
	*((uniform FVector *uniform)&S1) = *((uniform FVector *uniform)&B);

	Result = VectorCompareGE(S0, S1);

	return *((uniform FVector *uniform)&Result);
}

inline uniform FVector4 VectorCompareGT(const uniform FVector4 &A, const uniform FVector4 &B)
{
	varying FReal S0, S1, Result;
	*((uniform FVector4 *uniform)&S0) = *((uniform FVector4 *uniform)&A);
	*((uniform FVector4 *uniform)&S1) = *((uniform FVector4 *uniform)&B);

	Result = S0 > S1 ? REAL_BITS(NAN) : REAL_BITS(0);

	return *((uniform FVector4 *uniform)&Result);
}

inline uniform FVector VectorCompareGT(const uniform FVector& A, const uniform FVector& B)
{
	uniform FVector4 S0, S1, Result;
	*((uniform FVector *uniform)&S0) = *((uniform FVector *uniform)&A);
	*((uniform FVector *uniform)&S1) = *((uniform FVector *uniform)&B);

	Result = VectorCompareGT(S0, S1);

	return *((uniform FVector *uniform)&Result);
}

inline FVector4 VectorMax(const FVector4& V1, const FVector4& V2)
{
	FVector4 Result;

	Result.V[0] = max(V1.V[0], V2.V[0]);
	Result.V[1] = max(V1.V[1], V2.V[1]);
	Result.V[2] = max(V1.V[2], V2.V[2]);
	Result.V[3] = max(V1.V[3], V2.V[3]);

	return Result;
}

inline uniform FVector4 VectorMax(const uniform FVector4& V1, const uniform FVector4& V2)
{
	varying FReal S0, S1, Result;
	*((uniform FVector4 *uniform)&S0) = *((uniform FVector4 *uniform)&V1);
	*((uniform FVector4 *uniform)&S1) = *((uniform FVector4 *uniform)&V2);

	Result = max(S0, S1);

	return *((uniform FVector4 *uniform)&Result);
}

inline FVector3d VectorMax(const FVector3d& V1, const FVector3d& V2)
{
	FVector3d Result;

	Result.V[0] = max(V1.V[0], V2.V[0]);
	Result.V[1] = max(V1.V[1], V2.V[1]);
	Result.V[2] = max(V1.V[2], V2.V[2]);

	return Result;
}

inline FVector3f VectorMax(const FVector3f& V1, const FVector3f& V2)
{
	FVector3f Result;

	Result.V[0] = max(V1.V[0], V2.V[0]);
	Result.V[1] = max(V1.V[1], V2.V[1]);
	Result.V[2] = max(V1.V[2], V2.V[2]);

	return Result;
}

inline uniform FVector VectorMax(const uniform FVector& A, const uniform FVector& B)
{
	uniform FVector4 S0, S1, Result;
	*((uniform FVector *uniform)&S0) = *((uniform FVector *uniform)&A);
	*((uniform FVector *uniform)&S1) = *((uniform FVector *uniform)&B);

	Result = VectorMax(S0, S1);

	return *((uniform FVector *uniform)&Result);
}

inline uniform FVector8 VectorMax(const uniform FVector8 &V1, const uniform FVector8 &V2)
{
	varying float S0, S1, Result;
	*((uniform FVector8 *uniform)&S0) = *((uniform FVector8 *uniform)&V1);
	*((uniform FVector8 *uniform)&S1) = *((uniform FVector8 *uniform)&V2);

	Result = max(S0, S1);

	return *((uniform FVector8 *uniform)&Result);
}

inline FVector4 VectorMin(const FVector4& V1, const FVector4& V2)
{
	FVector4 Result;

	Result.V[0] = min(V1.V[0], V2.V[0]);
	Result.V[1] = min(V1.V[1], V2.V[1]);
	Result.V[2] = min(V1.V[2], V2.V[2]);
	Result.V[3] = min(V1.V[3], V2.V[3]);

	return Result;
}

inline uniform FVector4 VectorMin(const uniform FVector4& V1, const uniform FVector4& V2)
{
	varying FReal S0, S1, Result;
	*((uniform FVector4 *uniform)&S0) = *((uniform FVector4 *uniform)&V1);
	*((uniform FVector4 *uniform)&S1) = *((uniform FVector4 *uniform)&V2);

	Result = min(S0, S1);

	return *((uniform FVector4 *uniform)&Result);
}

inline FVector3d VectorMin(const FVector3d& V1, const FVector3d& V2)
{
	FVector3d Result;

	Result.V[0] = min(V1.V[0], V2.V[0]);
	Result.V[1] = min(V1.V[1], V2.V[1]);
	Result.V[2] = min(V1.V[2], V2.V[2]);

	return Result;
}

inline FVector3f VectorMin(const FVector3f& V1, const FVector3f& V2)
{
	FVector3f Result;

	Result.V[0] = min(V1.V[0], V2.V[0]);
	Result.V[1] = min(V1.V[1], V2.V[1]);
	Result.V[2] = min(V1.V[2], V2.V[2]);

	return Result;
}

inline uniform FVector VectorMin(const uniform FVector& A, const uniform FVector& B)
{
	uniform FVector4 S0, S1, Result;
	*((uniform FVector *uniform)&S0) = *((uniform FVector *uniform)&A);
	*((uniform FVector *uniform)&S1) = *((uniform FVector *uniform)&B);

	Result = VectorMin(S0, S1);

	return *((uniform FVector *uniform)&Result);
}

inline uniform FVector8 VectorMin(const uniform FVector8 &V1, const uniform FVector8 &V2)
{
	varying float S0, S1, Result;
	*((uniform FVector8 *uniform)&S0) = *((uniform FVector8 *uniform)&V1);
	*((uniform FVector8 *uniform)&S1) = *((uniform FVector8 *uniform)&V2);

	Result = min(S0, S1);

	return *((uniform FVector8 *uniform)&Result);
}

inline uniform FReal VectorSum(const uniform FVector4 &V)
{
	return V.V[0] + V.V[1] + V.V[2] + V.V[3];
}

inline FReal VectorSum(const FVector4 &V)
{
	return V.V[0] + V.V[1] + V.V[2] + V.V[3];
}

inline uniform FReal VectorSum(const uniform FVector &V)
{
	return V.V[0] + V.V[1] + V.V[2];
}

inline double VectorSum(const FVector3d &A)
{
	return A.V[0] + A.V[1] + A.V[2];
}

inline float VectorSum(const FVector3f &A)
{
	return A.V[0] + A.V[1] + A.V[2];
}

inline FVector4 VectorReciprocal(const FVector4& V)
{
	FVector4 Result;

	Result.V[0] = rcp_fast(V.V[0]);
	Result.V[1] = rcp_fast(V.V[1]);
	Result.V[2] = rcp_fast(V.V[2]);
	Result.V[3] = rcp_fast(V.V[3]);

	return Result;
}

inline uniform FVector4 VectorReciprocal(const uniform FVector4& V)
{
	varying FReal S0, Result;
	*((uniform FVector4 *uniform)&S0) = *((uniform FVector4 *uniform)&V);

	Result = rcp_fast(S0);

	return *((uniform FVector4 *uniform)&Result);
}

template <typename T>
inline T VectorReciprocalAccurate(const T& V)
{
	// Perform two passes of Newton-Raphson iteration on the hardware estimate
	//   x1 = x0 - f(x0) / f'(x0)
	//
	//    1 / Vec = x
	// => x * Vec = 1
	// => F(x) = x * Vec - 1
	//    F'(x) = Vec
	// => x1 = x0 - (x0 * Vec - 1) / Vec
	//
	// Since 1/Vec is what we're trying to solve, use an estimate for it, x0
	// => x1 = x0 - (x0 * Vec - 1) * x0 = 2 * x0 - Vec * x0^2

	// Initial estimate
	const T x0 = VectorReciprocal(V);

	// First iteration
	const T x0Squared = VectorMultiply(x0, x0);
	const T x0Times2 = VectorAdd(x0, x0);
	const T x1 = VectorSubtract(x0Times2, VectorMultiply(V, x0Squared));

	// Second iteration
	const T x1Squared = VectorMultiply(x1, x1);
	const T x1Times2 = VectorAdd(x1, x1);
	const T x2 = VectorSubtract(x1Times2, VectorMultiply(V, x1Squared));

	return x2;
}

inline FVector4f VectorReciprocalSqrt(const FVector4f& V)
{
	FVector4f Result;

	Result.V[0] = rsqrt_fast(V.V[0]);
	Result.V[1] = rsqrt_fast(V.V[1]);
	Result.V[2] = rsqrt_fast(V.V[2]);
	Result.V[3] = rsqrt_fast(V.V[3]);

	return Result;
}

inline FVector4d VectorReciprocalSqrt(const FVector4d& V)
{
	FVector4d Result;

	Result.V[0] = 1.0d / sqrt(V.V[0]);
	Result.V[1] = 1.0d / sqrt(V.V[1]);
	Result.V[2] = 1.0d / sqrt(V.V[2]);
	Result.V[3] = 1.0d / sqrt(V.V[3]);

	return Result;
}

inline uniform FVector4f VectorReciprocalSqrt(const uniform FVector4f& V)
{
	varying float S0, Result;
	*((uniform FVector4f *uniform)&S0) = *((uniform FVector4f *uniform)&V);

	Result = rsqrt_fast(S0);

	return *((uniform FVector4f *uniform)&Result);
}

inline uniform FVector4d VectorReciprocalSqrt(const uniform FVector4d& V)
{
	varying double S0, Result;
	*((uniform FVector4d *uniform)&S0) = *((uniform FVector4d *uniform)&V);

	Result = 1.0d / sqrt(S0);

	return *((uniform FVector4d *uniform)&Result);
}

inline FVector4f VectorReciprocalSqrtAccurate(const FVector4f& V)
{
	// Perform two passes of Newton-Raphson iteration on the hardware estimate
	//    v^-0.5 = x
	// => x^2 = v^-1
	// => 1/(x^2) = v
	// => F(x) = x^-2 - v
	//    F'(x) = -2x^-3

	//    x1 = x0 - F(x0)/F'(x0)
	// => x1 = x0 + 0.5 * (x0^-2 - Vec) * x0^3
	// => x1 = x0 + 0.5 * (x0 - Vec * x0^3)
	// => x1 = x0 + x0 * (0.5 - 0.5 * Vec * x0^2)

	const FVector4f VecDivBy2 = V * FloatOneHalf;

	// Initial estimate
	const FVector4f x0 = VectorReciprocalSqrt(V);

	// First iteration
	FVector4f x1 = x0 * x0;
	x1 = FloatOneHalf - (VecDivBy2 * x1);
	x1 = x0 * x1 + x0;

	// Second iteration
	FVector4f x2 = x1 * x1;
	x2 = FloatOneHalf - (VecDivBy2 * x2);
	x2 = x1 * x2 + x1;

	return x2;
}

inline FVector4d VectorReciprocalSqrtAccurate(const FVector4d& V)
{
	FVector4d Result;

	Result.V[0] = 1.0d / sqrt(V.V[0]);
	Result.V[1] = 1.0d / sqrt(V.V[1]);
	Result.V[2] = 1.0d / sqrt(V.V[2]);
	Result.V[3] = 1.0d / sqrt(V.V[3]);

	return Result;
}

inline uniform FVector4f VectorReciprocalSqrtAccurate(const uniform FVector4f& V)
{
	// Perform two passes of Newton-Raphson iteration on the hardware estimate
	//    v^-0.5 = x
	// => x^2 = v^-1
	// => 1/(x^2) = v
	// => F(x) = x^-2 - v
	//    F'(x) = -2x^-3

	//    x1 = x0 - F(x0)/F'(x0)
	// => x1 = x0 + 0.5 * (x0^-2 - Vec) * x0^3
	// => x1 = x0 + 0.5 * (x0 - Vec * x0^3)
	// => x1 = x0 + x0 * (0.5 - 0.5 * Vec * x0^2)

	const uniform FVector4f VecDivBy2 = V * FloatOneHalf;

	// Initial estimate
	const uniform FVector4f x0 = VectorReciprocalSqrt(V);

	// First iteration
	uniform FVector4f x1 = x0 * x0;
	x1 = FloatOneHalf - (VecDivBy2 * x1);
	x1 = x0 * x1 + x0;

	// Second iteration
	uniform FVector4f x2 = x1 * x1;
	x2 = FloatOneHalf - (VecDivBy2 * x2);
	x2 = x1 * x2 + x1;

	return x2;
}

inline uniform FVector4d VectorReciprocalSqrtAccurate(const uniform FVector4d& V)
{
	varying double S0, Result;
	*((uniform FVector4d *uniform)&S0) = *((uniform FVector4d *uniform)&V);

	Result = 1.0d / sqrt(S0);

	return *((uniform FVector4d *uniform)&Result);
}

inline uniform FVector VectorReciprocalSqrtAccurate(const uniform FVector& V)
{
	uniform FVector4 S0, Result;
	*((uniform FVector *uniform)&S0) = *((uniform FVector *uniform)&V);

	Result = VectorReciprocalSqrtAccurate(S0);

	return *((uniform FVector *uniform)&Result);
}

/**
 * Replicates one element into all four elements and returns the new vector.
 *
 * @param Vec			Source vector
 * @param ElementIndex	Index (0-3) of the element to replicate
 * @return				VectorRegister( Vec[ElementIndex], Vec[ElementIndex], Vec[ElementIndex], Vec[ElementIndex] )
 */
template<typename T>
inline T VectorReplicate(const T& Vec, const uniform int ElementIndex)
{
	return SetVector4(Vec.V[ElementIndex], Vec.V[ElementIndex], Vec.V[ElementIndex], Vec.V[ElementIndex]);
}

template<>
inline uniform FVector8 VectorReplicate<uniform FVector8>(const uniform FVector8 &Vec, const uniform int R)
{
	return SetVector8(Vec.V[R], Vec.V[R], Vec.V[R], Vec.V[R], Vec.V[R+4], Vec.V[R+4], Vec.V[R+4], Vec.V[R+4]);
}

/**
 * Swizzles the 4 components of a vector and returns the result.
 *
 * @param Vec		Source vector
 * @param X			Index for which component to use for X (literal 0-3)
 * @param Y			Index for which component to use for Y (literal 0-3)
 * @param Z			Index for which component to use for Z (literal 0-3)
 * @param W			Index for which component to use for W (literal 0-3)
 * @return			The swizzled vector
 */
inline FVector4d VectorSwizzle(const FVector4d &Vec, const uniform int X, const uniform int Y, const uniform int Z, const uniform int W)
{
	return SetVector4(Vec.V[X], Vec.V[Y], Vec.V[Z], Vec.V[W]);
}

inline FVector4f VectorSwizzle(const FVector4f &Vec, const uniform int X, const uniform int Y, const uniform int Z, const uniform int W)
{
	return SetVector4(Vec.V[X], Vec.V[Y], Vec.V[Z], Vec.V[W]);
}

inline uniform FVector4d VectorSwizzle(const uniform FVector4d &Vec, const uniform int X, const uniform int Y, const uniform int Z, const uniform int W)
{
	return SetVector4(Vec.V[X], Vec.V[Y], Vec.V[Z], Vec.V[W]);
}

inline uniform FVector4f VectorSwizzle(const uniform FVector4f &Vec, const uniform int X, const uniform int Y, const uniform int Z, const uniform int W)
{
	return SetVector4(Vec.V[X], Vec.V[Y], Vec.V[Z], Vec.V[W]);
}

inline FVector3d VectorSwizzle(const FVector3d &Vec, const uniform int X, const uniform int Y, const uniform int Z)
{
	return SetVector(Vec.V[X], Vec.V[Y], Vec.V[Z]);
}

inline FVector3f VectorSwizzle(const FVector3f &Vec, const uniform int X, const uniform int Y, const uniform int Z)
{
	return SetVector(Vec.V[X], Vec.V[Y], Vec.V[Z]);
}

inline uniform FVector3d VectorSwizzle(const uniform FVector3d &Vec, const uniform int X, const uniform int Y, const uniform int Z)
{
	return SetVector(Vec.V[X], Vec.V[Y], Vec.V[Z]);
}

inline uniform FVector3f VectorSwizzle(const uniform FVector3f &Vec, const uniform int X, const uniform int Y, const uniform int Z)
{
	return SetVector(Vec.V[X], Vec.V[Y], Vec.V[Z]);
}

inline uniform FVector8 VectorSwizzle(const uniform FVector8 &Vec, const uniform int X, const uniform int Y, const uniform int Z, const uniform int W)
{
	return SetVector8(Vec.V[X], Vec.V[Y], Vec.V[Z], Vec.V[W], Vec.V[X+4], Vec.V[Y+4], Vec.V[Z+4], Vec.V[W+4]);
}

/**
 * Creates a vector through selecting two components from each vector via a shuffle mask. 
 *
 * @param Vec1		Source vector1
 * @param Vec2		Source vector2
 * @param X			Index for which component of Vector1 to use for X (literal 0-3)
 * @param Y			Index for which component of Vector1 to use for Y (literal 0-3)
 * @param Z			Index for which component of Vector2 to use for Z (literal 0-3)
 * @param W			Index for which component of Vector2 to use for W (literal 0-3)
 * @return			The swizzled vector
 */
template <typename T>
inline T VectorShuffle(const T& Vec1, const T& Vec2, const uniform int X, const uniform int Y, const uniform int Z, const uniform int W)
{
	return SetVector4(Vec1.V[X], Vec1.V[Y], Vec2.V[Z], Vec2.V[W]);
}

/**
 * Calculates the cross product of two vectors (XYZ components). W is set to 0.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		cross(Vec1.xyz, Vec2.xyz). W is set to 0.
 */
inline FVector4d VectorCross(const FVector4d& Vec1, const FVector4d& Vec2)
{
	FVector4d Tmp0 = VectorSwizzle(Vec2, 1,2,0,3);
	FVector4d Tmp1 = VectorSwizzle(Vec1, 1,2,0,3);
	Tmp0 = Tmp0 * Vec1;
	Tmp1 = Tmp1 * Vec2;
	FVector4d Tmp2 = Tmp0 - Tmp1;
	return VectorSwizzle(Tmp2, 1,2,0,3);
}

inline FVector4f VectorCross(const FVector4f& Vec1, const FVector4f& Vec2)
{
	FVector4f Tmp0 = VectorSwizzle(Vec2, 1,2,0,3);
	FVector4f Tmp1 = VectorSwizzle(Vec1, 1,2,0,3);
	Tmp0 = Tmp0 * Vec1;
	Tmp1 = Tmp1 * Vec2;
	FVector4f Tmp2 = Tmp0 - Tmp1;
	return VectorSwizzle(Tmp2, 1,2,0,3);
}

inline FVector4f VectorCross(const uniform FVector4f& Vec1, const FVector4f& Vec2)
{
	FVector4f Tmp0 = VectorSwizzle(Vec2, 1,2,0,3);
	FVector4f Tmp1 = VectorSwizzle(Vec1, 1,2,0,3);
	Tmp0 = Tmp0 * Vec1;
	Tmp1 = Tmp1 * Vec2;
	FVector4f Tmp2 = Tmp0 - Tmp1;
	return VectorSwizzle(Tmp2, 1,2,0,3);
}

inline FVector4 VectorCross(const uniform FVector4& Vec1, const FVector4& Vec2)
{
	FVector4 Tmp0 = VectorSwizzle(Vec2, 1,2,0,3);
	FVector4 Tmp1 = VectorSwizzle(Vec1, 1,2,0,3);
	Tmp0 = Tmp0 * Vec1;
	Tmp1 = Tmp1 * Vec2;
	FVector4 Tmp2 = Tmp0 - Tmp1;
	return VectorSwizzle(Tmp2, 1,2,0,3);
}

inline uniform FVector4d VectorCross(const uniform FVector4d& Vec1, const uniform FVector4d& Vec2)
{
	uniform FVector4d Tmp0 = VectorSwizzle(Vec2, 1,2,0,3);
	uniform FVector4d Tmp1 = VectorSwizzle(Vec1, 1,2,0,3);
	Tmp0 = Tmp0 * Vec1;
	Tmp1 = Tmp1 * Vec2;
	uniform FVector4d Tmp2 = Tmp0 - Tmp1;
	return VectorSwizzle(Tmp2, 1,2,0,3);
}

inline uniform FVector4f VectorCross(const uniform FVector4f& Vec1, const uniform FVector4f& Vec2)
{
	uniform FVector4f Tmp0 = VectorSwizzle(Vec2, 1,2,0,3);
	uniform FVector4f Tmp1 = VectorSwizzle(Vec1, 1,2,0,3);
	Tmp0 = Tmp0 * Vec1;
	Tmp1 = Tmp1 * Vec2;
	uniform FVector4f Tmp2 = Tmp0 - Tmp1;
	return VectorSwizzle(Tmp2, 1,2,0,3);
}

inline FVector3d VectorCross(const FVector3d& Vec1, const FVector3d& Vec2)
{
	FVector3d Tmp0 = VectorSwizzle(Vec2, 1,2,0);
	FVector3d Tmp1 = VectorSwizzle(Vec1, 1,2,0);
	Tmp0 = Tmp0 * Vec1;
	Tmp1 = Tmp1 * Vec2;
	const FVector3d Tmp2 = Tmp0 - Tmp1;
	return VectorSwizzle(Tmp2, 1,2,0);
}

inline FVector3f VectorCross(const FVector3f& Vec1, const FVector3f& Vec2)
{
	FVector3f Tmp0 = VectorSwizzle(Vec2, 1,2,0);
	FVector3f Tmp1 = VectorSwizzle(Vec1, 1,2,0);
	Tmp0 = Tmp0 * Vec1;
	Tmp1 = Tmp1 * Vec2;
	const FVector3f Tmp2 = Tmp0 - Tmp1;
	return VectorSwizzle(Tmp2, 1,2,0);
}

inline FVector3d VectorCross(const uniform FVector3d& Vec1, const FVector3d& Vec2)
{
	FVector3d Tmp0 = VectorSwizzle(Vec2, 1,2,0);
	const uniform FVector3d Tmp1 = VectorSwizzle(Vec1, 1,2,0);
	Tmp0 = Tmp0 * Vec1;
	const FVector3d Tmp3 = Tmp1 * Vec2;
	const FVector3d Tmp2 = Tmp0 - Tmp3;
	return VectorSwizzle(Tmp2, 1,2,0);
}

inline FVector3f VectorCross(const uniform FVector3f& Vec1, const FVector3f& Vec2)
{
	FVector3f Tmp0 = VectorSwizzle(Vec2, 1,2,0);
	const uniform FVector3f Tmp1 = VectorSwizzle(Vec1, 1,2,0);
	Tmp0 = Tmp0 * Vec1;
	const FVector3f Tmp3 = Tmp1 * Vec2;
	const FVector3f Tmp2 = Tmp0 - Tmp3;
	return VectorSwizzle(Tmp2, 1,2,0);
}

inline uniform FVector3d VectorCross(const uniform FVector3d& Vec1, const uniform FVector3d& Vec2)
{
	uniform FVector3d Tmp0 = VectorSwizzle(Vec2, 1,2,0);
	uniform FVector3d Tmp1 = VectorSwizzle(Vec1, 1,2,0);
	Tmp0 = Tmp0 * Vec1;
	Tmp1 = Tmp1 * Vec2;
	const uniform FVector3d Tmp2 = Tmp0 - Tmp1;
	return VectorSwizzle(Tmp2, 1,2,0);
}

inline uniform FVector3f VectorCross(const uniform FVector3f& Vec1, const uniform FVector3f& Vec2)
{
	uniform FVector3f Tmp0 = VectorSwizzle(Vec2, 1,2,0);
	uniform FVector3f Tmp1 = VectorSwizzle(Vec1, 1,2,0);
	Tmp0 = Tmp0 * Vec1;
	Tmp1 = Tmp1 * Vec2;
	const uniform FVector3f Tmp2 = Tmp0 - Tmp1;
	return VectorSwizzle(Tmp2, 1,2,0);
}

inline uniform FVector8 VectorCross(const uniform FVector8& Vec1, const uniform FVector8& Vec2)
{
	uniform FVector8 Tmp0 = VectorSwizzle(Vec2, 1,2,0,3);
	uniform FVector8 Tmp1 = VectorSwizzle(Vec1, 1,2,0,3);
	Tmp0 = Tmp0 * Vec1;
	Tmp1 = Tmp1 * Vec2;
	uniform FVector8 Tmp2 = Tmp0 - Tmp1;
	return VectorSwizzle(Tmp2, 1,2,0,3);
}

/**
 * Calculates the dot4 product of two vectors and returns a vector with the result in all 4 components.
 * Only really efficient on Xbox 360.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		d = dot4(Vec1.xyzw, Vec2.xyzw), VectorRegister( d, d, d, d )
 */
inline FVector4d VectorDot4( const FVector4d& Vec1, const FVector4d& Vec2 )
{
	FVector4d Temp1, Temp2;
	Temp1 = Vec1 * Vec2;
	Temp2 = VectorSwizzle(Temp1, 2,3,0,1);	// (Z,W,X,Y).
	Temp1 = Temp1 + Temp2; // (X*X + Z*Z, Y*Y + W*W, Z*Z + X*X, W*W + Y*Y)
	Temp2 = VectorSwizzle(Temp1, 1,2,3,0); // Rotate left 4 bytes (Y,Z,W,X).
	return Temp1 + Temp2; // (X*X + Z*Z + Y*Y + W*W, Y*Y + W*W + Z*Z + X*X, Z*Z + X*X + W*W + Y*Y, W*W + Y*Y + X*X + Z*Z)
}

inline FVector4f VectorDot4( const FVector4f& Vec1, const FVector4f& Vec2 )
{
	FVector4f Temp1, Temp2;
	Temp1 = Vec1 * Vec2;
	Temp2 = VectorSwizzle(Temp1, 2,3,0,1);	// (Z,W,X,Y).
	Temp1 = Temp1 + Temp2; // (X*X + Z*Z, Y*Y + W*W, Z*Z + X*X, W*W + Y*Y)
	Temp2 = VectorSwizzle(Temp1, 1,2,3,0); // Rotate left 4 bytes (Y,Z,W,X).
	return Temp1 + Temp2; // (X*X + Z*Z + Y*Y + W*W, Y*Y + W*W + Z*Z + X*X, Z*Z + X*X + W*W + Y*Y, W*W + Y*Y + X*X + Z*Z)
}

inline uniform FVector4 VectorDot4( const uniform FVector4& Vec1, const uniform FVector4& Vec2 )
{
	uniform FVector4 Temp1, Temp2;
	Temp1 = Vec1 * Vec2;
	Temp2 = VectorSwizzle(Temp1, 2,3,0,1);	// (Z,W,X,Y).
	Temp1 = Temp1 + Temp2; // (X*X + Z*Z, Y*Y + W*W, Z*Z + X*X, W*W + Y*Y)
	Temp2 = VectorSwizzle(Temp1, 1,2,3,0); // Rotate left 4 bytes (Y,Z,W,X).
	return Temp1 + Temp2; // (X*X + Z*Z + Y*Y + W*W, Y*Y + W*W + Z*Z + X*X, Z*Z + X*X + W*W + Y*Y, W*W + Y*Y + X*X + Z*Z)
}

inline double VectorDot( const FVector4d& Vec1, const FVector4d& Vec2 )
{
	FVector4d Temp1, Temp2;
	Temp1 = Vec1 * Vec2;
	Temp2 = VectorSwizzle(Temp1, 2,3,0,1);	// (Z,W,X,Y).
	Temp1 = Temp1 + Temp2; // (X*X + Z*Z, Y*Y + W*W, Z*Z + X*X, W*W + Y*Y)
	Temp2 = VectorSwizzle(Temp1, 1,2,3,0); // Rotate left 4 bytes (Y,Z,W,X).
	return Temp1.V[0] + Temp2.V[0]; // (X*X + Z*Z + Y*Y + W*W)
}

inline float VectorDot( const FVector4f& Vec1, const FVector4f& Vec2 )
{
	FVector4f Temp1, Temp2;
	Temp1 = Vec1 * Vec2;
	Temp2 = VectorSwizzle(Temp1, 2,3,0,1);	// (Z,W,X,Y).
	Temp1 = Temp1 + Temp2; // (X*X + Z*Z, Y*Y + W*W, Z*Z + X*X, W*W + Y*Y)
	Temp2 = VectorSwizzle(Temp1, 1,2,3,0); // Rotate left 4 bytes (Y,Z,W,X).
	return Temp1.V[0] + Temp2.V[0]; // (X*X + Z*Z + Y*Y + W*W)
}

inline uniform FReal VectorDot( const uniform FVector4& Vec1, const uniform FVector4& Vec2 )
{
	uniform FVector4 Temp1, Temp2;
	Temp1 = Vec1 * Vec2;
	Temp2 = VectorSwizzle(Temp1, 2,3,0,1);	// (Z,W,X,Y).
	Temp1 = Temp1 + Temp2; // (X*X + Z*Z, Y*Y + W*W, Z*Z + X*X, W*W + Y*Y)
	Temp2 = VectorSwizzle(Temp1, 1,2,3,0); // Rotate left 4 bytes (Y,Z,W,X).
	return Temp1.V[0] + Temp2.V[0]; // (X*X + Z*Z + Y*Y + W*W)
}

inline uniform double VectorDot( const uniform FVector3d &Vec1, const uniform FVector3d &Vec2 )
{
	const uniform FVector3d Temp = Vec1 * Vec2;
	return Temp.V[0] + Temp.V[1] + Temp.V[2];
}

inline uniform float VectorDot( const uniform FVector3f &Vec1, const uniform FVector3f &Vec2 )
{
	const uniform FVector3f Temp = Vec1 * Vec2;
	return Temp.V[0] + Temp.V[1] + Temp.V[2];
}

inline varying double VectorDot( const varying FVector3d &Vec1, const varying FVector3d &Vec2 )
{
	const varying FVector3d Temp = Vec1 * Vec2;
	return Temp.V[0] + Temp.V[1] + Temp.V[2];
}

inline varying float VectorDot( const varying FVector3f &Vec1, const varying FVector3f &Vec2 )
{
	const varying FVector3f Temp = Vec1 * Vec2;
	return Temp.V[0] + Temp.V[1] + Temp.V[2];
}

inline varying double VectorDot( const varying FVector3d &Vec1, const uniform FVector3d &Vec2 )
{
	const varying FVector3d Temp = Vec1 * Vec2;
	return Temp.V[0] + Temp.V[1] + Temp.V[2];
}

inline varying float VectorDot( const varying FVector3f &Vec1, const uniform FVector3f &Vec2 )
{
	const varying FVector3f Temp = Vec1 * Vec2;
	return Temp.V[0] + Temp.V[1] + Temp.V[2];
}

inline uniform FVector4 VectorBitwiseAnd(const uniform FVector4 &Vec1, const uniform FVector4 &Vec2)
{
	varying FReal S0, S1, Result;
	*((uniform FVector4 *uniform)&S0) = *((uniform FVector4 *uniform)&Vec1);
	*((uniform FVector4 *uniform)&S1) = *((uniform FVector4 *uniform)&Vec2);

	Result = REAL_BITS(intbits(S0) & intbits(S1));

	return *((uniform FVector4 *uniform)&Result);
}

inline uniform FVector VectorBitwiseAnd(const uniform FVector& A, const uniform FVector& B)
{
	uniform FVector4 S0, S1, Result;
	*((uniform FVector *uniform)&S0) = *((uniform FVector *uniform)&A);
	*((uniform FVector *uniform)&S1) = *((uniform FVector *uniform)&B);

	Result = VectorBitwiseAnd(S0, S1);

	return *((uniform FVector *uniform)&Result);
}

inline uniform FVector4 VectorBitwiseAnd(const uniform FVector4 &Vec1, const uniform UIntType I)
{
	varying FReal S0, Result;
	*((uniform FVector4 *uniform)&S0) = *((uniform FVector4 *uniform)&Vec1);

	Result = REAL_BITS(intbits(S0) & I);

	return *((uniform FVector4 *uniform)&Result);
}

inline uniform FVector VectorBitwiseAnd(const uniform FVector& A, const uniform unsigned int I)
{
	uniform FVector4 S0, Result;
	*((uniform FVector *uniform)&S0) = *((uniform FVector *uniform)&A);

	Result = VectorBitwiseAnd(S0, I);

	return *((uniform FVector *uniform)&Result);
}

inline uniform FVector4 VectorBitwiseOr(const uniform FVector4 &Vec1, const uniform FVector4 &Vec2)
{
	varying FReal S0, S1, Result;
	*((uniform FVector4 *uniform)&S0) = *((uniform FVector4 *uniform)&Vec1);
	*((uniform FVector4 *uniform)&S1) = *((uniform FVector4 *uniform)&Vec2);

	Result = REAL_BITS(intbits(S0) | intbits(S1));

	return *((uniform FVector4 *uniform)&Result);
}

inline uniform FVector VectorBitwiseOr(const uniform FVector& A, const uniform FVector& B)
{
	uniform FVector4 S0, S1, Result;
	*((uniform FVector *uniform)&S0) = *((uniform FVector *uniform)&A);
	*((uniform FVector *uniform)&S1) = *((uniform FVector *uniform)&B);

	Result = VectorBitwiseOr(S0, S1);

	return *((uniform FVector *uniform)&Result);
}

inline uniform FVector4 VectorBitwiseXor(const uniform FVector4 &Vec1, const uniform FVector4 &Vec2)
{
	varying FReal S0, S1, Result;
	*((uniform FVector4 *uniform)&S0) = *((uniform FVector4 *uniform)&Vec1);
	*((uniform FVector4 *uniform)&S1) = *((uniform FVector4 *uniform)&Vec2);

	Result = REAL_BITS(intbits(S0) ^ intbits(S1));

	return *((uniform FVector4 *uniform)&Result);
}

inline uniform FVector VectorBitwiseXor(const uniform FVector& A, const uniform FVector& B)
{
	uniform FVector4 S0, S1, Result;
	*((uniform FVector *uniform)&S0) = *((uniform FVector *uniform)&A);
	*((uniform FVector *uniform)&S1) = *((uniform FVector *uniform)&B);

	Result = VectorBitwiseXor(S0, S1);

	return *((uniform FVector *uniform)&Result);
}

/**
 * Does a bitwise vector selection based on a mask (e.g., created from VectorCompareXX)
 *
 * @param Mask  Mask (when 1: use the corresponding bit from Vec1 otherwise from Vec2)
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister( for each bit i: Mask[i] ? Vec1[i] : Vec2[i] )
 *
 */
inline FVector4 VectorSelect(const FVector4& Mask, const FVector4& Vec1, const FVector4& Vec2 )
{
	FVector4 Result;

	Result.V[0] = REAL_BITS(intbits(Vec2.V[0]) ^ (intbits(Mask.V[0]) & (intbits(Vec1.V[0]) ^ intbits(Vec2.V[0]))));
	Result.V[1] = REAL_BITS(intbits(Vec2.V[1]) ^ (intbits(Mask.V[1]) & (intbits(Vec1.V[1]) ^ intbits(Vec2.V[1]))));
	Result.V[2] = REAL_BITS(intbits(Vec2.V[2]) ^ (intbits(Mask.V[2]) & (intbits(Vec1.V[2]) ^ intbits(Vec2.V[2]))));
	Result.V[3] = REAL_BITS(intbits(Vec2.V[3]) ^ (intbits(Mask.V[3]) & (intbits(Vec1.V[3]) ^ intbits(Vec2.V[3]))));

	return Result;
}

inline FVector4d VectorSelect(const FVector4d& Mask, const FVector4d& Vec1, const uniform FVector4d& Vec2 )
{
	FVector4d Result;

	Result.V[0] = doublebits(intbits(Vec2.V[0]) ^ (intbits(Mask.V[0]) & (intbits(Vec1.V[0]) ^ intbits(Vec2.V[0]))));
	Result.V[1] = doublebits(intbits(Vec2.V[1]) ^ (intbits(Mask.V[1]) & (intbits(Vec1.V[1]) ^ intbits(Vec2.V[1]))));
	Result.V[2] = doublebits(intbits(Vec2.V[2]) ^ (intbits(Mask.V[2]) & (intbits(Vec1.V[2]) ^ intbits(Vec2.V[2]))));
	Result.V[3] = doublebits(intbits(Vec2.V[3]) ^ (intbits(Mask.V[3]) & (intbits(Vec1.V[3]) ^ intbits(Vec2.V[3]))));

	return Result;
}

inline FVector4f VectorSelect(const FVector4f& Mask, const FVector4f& Vec1, const uniform FVector4f& Vec2 )
{
	FVector4f Result;

	Result.V[0] = floatbits(intbits(Vec2.V[0]) ^ (intbits(Mask.V[0]) & (intbits(Vec1.V[0]) ^ intbits(Vec2.V[0]))));
	Result.V[1] = floatbits(intbits(Vec2.V[1]) ^ (intbits(Mask.V[1]) & (intbits(Vec1.V[1]) ^ intbits(Vec2.V[1]))));
	Result.V[2] = floatbits(intbits(Vec2.V[2]) ^ (intbits(Mask.V[2]) & (intbits(Vec1.V[2]) ^ intbits(Vec2.V[2]))));
	Result.V[3] = floatbits(intbits(Vec2.V[3]) ^ (intbits(Mask.V[3]) & (intbits(Vec1.V[3]) ^ intbits(Vec2.V[3]))));

	return Result;
}

inline FVector4 VectorSelect(const FVector4& Mask, const uniform FVector4& Vec1, const FVector4& Vec2 )
{
	FVector4 Result;

	Result.V[0] = REAL_BITS(intbits(Vec2.V[0]) ^ (intbits(Mask.V[0]) & (intbits(Vec1.V[0]) ^ intbits(Vec2.V[0]))));
	Result.V[1] = REAL_BITS(intbits(Vec2.V[1]) ^ (intbits(Mask.V[1]) & (intbits(Vec1.V[1]) ^ intbits(Vec2.V[1]))));
	Result.V[2] = REAL_BITS(intbits(Vec2.V[2]) ^ (intbits(Mask.V[2]) & (intbits(Vec1.V[2]) ^ intbits(Vec2.V[2]))));
	Result.V[3] = REAL_BITS(intbits(Vec2.V[3]) ^ (intbits(Mask.V[3]) & (intbits(Vec1.V[3]) ^ intbits(Vec2.V[3]))));

	return Result;
}

inline uniform FVector4 VectorSelect(const uniform FVector4& Mask, const uniform FVector4& Vec1, const uniform FVector4& Vec2 )
{
	return VectorBitwiseXor(Vec2, VectorBitwiseAnd(Mask, VectorBitwiseXor(Vec1, Vec2)));
}

inline uniform FVector VectorSelect(const uniform FVector& Mask, const uniform FVector& Vec1, const uniform FVector& Vec2 )
{
	return VectorBitwiseXor(Vec2, VectorBitwiseAnd(Mask, VectorBitwiseXor(Vec1, Vec2)));
}

inline FVector3d VectorSelect(varying bool Mask, const varying FVector3d &Vec1, const varying FVector3d &Vec2)
{
	FVector3d Result;

	Result.V[0] = select(Mask, Vec1.V[0], Vec2.V[0]);
	Result.V[1] = select(Mask, Vec1.V[1], Vec2.V[1]);
	Result.V[2] = select(Mask, Vec1.V[2], Vec2.V[2]);

	return Result;
}

inline FVector3f VectorSelect(varying bool Mask, const varying FVector3f &Vec1, const varying FVector3f &Vec2)
{
	FVector3f Result;

	Result.V[0] = select(Mask, Vec1.V[0], Vec2.V[0]);
	Result.V[1] = select(Mask, Vec1.V[1], Vec2.V[1]);
	Result.V[2] = select(Mask, Vec1.V[2], Vec2.V[2]);

	return Result;
}

inline FVector3d VectorSelect(varying bool Mask, const uniform FVector3d &Vec1, const varying FVector3d &Vec2)
{
	FVector3d Result;

	Result.V[0] = select(Mask, Vec1.V[0], Vec2.V[0]);
	Result.V[1] = select(Mask, Vec1.V[1], Vec2.V[1]);
	Result.V[2] = select(Mask, Vec1.V[2], Vec2.V[2]);

	return Result;
}

inline FVector3f VectorSelect(varying bool Mask, const uniform FVector3f &Vec1, const varying FVector3f &Vec2)
{
	FVector3f Result;

	Result.V[0] = select(Mask, Vec1.V[0], Vec2.V[0]);
	Result.V[1] = select(Mask, Vec1.V[1], Vec2.V[1]);
	Result.V[2] = select(Mask, Vec1.V[2], Vec2.V[2]);

	return Result;
}

inline FVector3d VectorSelect(varying bool Mask, const varying FVector3d &Vec1, const uniform FVector3d &Vec2)
{
	FVector3d Result;

	Result.V[0] = select(Mask, Vec1.V[0], Vec2.V[0]);
	Result.V[1] = select(Mask, Vec1.V[1], Vec2.V[1]);
	Result.V[2] = select(Mask, Vec1.V[2], Vec2.V[2]);

	return Result;
}

inline FVector3f VectorSelect(varying bool Mask, const varying FVector3f &Vec1, const uniform FVector3f &Vec2)
{
	FVector3f Result;

	Result.V[0] = select(Mask, Vec1.V[0], Vec2.V[0]);
	Result.V[1] = select(Mask, Vec1.V[1], Vec2.V[1]);
	Result.V[2] = select(Mask, Vec1.V[2], Vec2.V[2]);

	return Result;
}

inline uniform FVector3d VectorSelect(uniform bool Mask, const uniform FVector3d &Vec1, const uniform FVector3d &Vec2)
{
	uniform FVector3d Result;

	Result.V[0] = select(Mask, Vec1.V[0], Vec2.V[0]);
	Result.V[1] = select(Mask, Vec1.V[1], Vec2.V[1]);
	Result.V[2] = select(Mask, Vec1.V[2], Vec2.V[2]);

	return Result;
}

inline uniform FVector3f VectorSelect(uniform bool Mask, const uniform FVector3f &Vec1, const uniform FVector3f &Vec2)
{
	uniform FVector3f Result;

	Result.V[0] = select(Mask, Vec1.V[0], Vec2.V[0]);
	Result.V[1] = select(Mask, Vec1.V[1], Vec2.V[1]);
	Result.V[2] = select(Mask, Vec1.V[2], Vec2.V[2]);

	return Result;
}

inline uniform FVector4 VectorSign(const uniform FVector4& X)
{
	const uniform FVector4 Mask = VectorCompareGE(X, (VectorZero));
	return VectorSelect(Mask, VectorOne, VectorMinusOne);
}

// Returns ((Vector dot Vector) >= 1e-8) ? (Vector / |Vector|) : DefaultValue
// Uses accurate 1/sqrt, not the estimate
inline FVector4d VectorNormalizeSafe( const FVector4d& Vector, const uniform FVector4d& DefaultValue)
{
	const FVector4d SquareSum = VectorDot4(Vector, Vector);
	const FVector4d NonZeroMask = VectorCompareGE(SquareSum, DoubleSmallLengthThreshold);
	const FVector4d InvLength = VectorReciprocalSqrtAccurate(SquareSum);
	const FVector4d NormalizedVector = InvLength * Vector;
	return VectorSelect(NonZeroMask, NormalizedVector, DefaultValue);
}

inline FVector4f VectorNormalizeSafe( const FVector4f& Vector, const uniform FVector4f& DefaultValue)
{
	const FVector4f SquareSum = VectorDot4(Vector, Vector);
	const FVector4f NonZeroMask = VectorCompareGE(SquareSum, FloatSmallLengthThreshold);
	const FVector4f InvLength = VectorReciprocalSqrtAccurate(SquareSum);
	const FVector4f NormalizedVector = InvLength * Vector;
	return VectorSelect(NonZeroMask, NormalizedVector, DefaultValue);
}

inline uniform FVector4 VectorNormalizeSafe( const uniform FVector4& Vector, const uniform FVector4& DefaultValue)
{
	const uniform FVector4 SquareSum = VectorDot4(Vector, Vector);
	const uniform FVector4 NonZeroMask = VectorCompareGE(SquareSum, SmallLengthThreshold);
	const uniform FVector4 InvLength = VectorReciprocalSqrtAccurate(SquareSum);
	const uniform FVector4 NormalizedVector = InvLength * Vector;
	return VectorSelect(NonZeroMask, NormalizedVector, DefaultValue);
}

inline uniform FVector4 VectorNormalizeSafe2( const uniform FVector4& Vector, const uniform FVector4& DefaultValue)
{
	const uniform FReal SquareSum = VectorDot(Vector, Vector);
	const uniform FReal NonZeroMask = VectorCompareGE(SquareSum, SMALL_NUMBER);
	const uniform FReal InvLength = ReciprocalSqrtAccurate(SquareSum);
	const uniform FVector4 NormalizedVector = Vector * InvLength;
	const uniform FVector4 VNonZeroMask = SetVector4(NonZeroMask, NonZeroMask, NonZeroMask, NonZeroMask);
	return VectorSelect(VNonZeroMask, NormalizedVector, DefaultValue);
}

inline FVector VectorNormalize(const varying FVector& Vector)
{
	const varying FReal SquareSum = VectorDot(Vector, Vector);
	const varying bool bNonZeroMask = SquareSum > SMALL_NUMBER;
	const varying FReal Scale = ReciprocalSqrtAccurate(SquareSum);
	return VectorSelect(bNonZeroMask, Vector * Scale, Vector);
}

inline uniform FVector VectorNormalize(const uniform FVector& Vector)
{
	const uniform FReal SquareSum = VectorDot(Vector, Vector);
	const uniform bool bNonZeroMask = SquareSum > SMALL_NUMBER;
	const uniform FReal Scale = ReciprocalSqrtAccurate(SquareSum);
	return VectorSelect(bNonZeroMask, Vector * Scale, Vector);
}

inline uniform FVector4 VectorTruncate(const uniform FVector4 &X)
{
	varying FReal S0, Result;
	*((uniform FVector4 *uniform)&S0) = *((uniform FVector4 *uniform)&X);

	Result = trunc(S0);

	return *((uniform FVector4 *uniform)&Result);
}

inline uniform FVector4 VectorNegate(const uniform FVector4 &A)
{
	varying FReal S0, Result;
	*((uniform FVector4 *uniform)&S0) = *((uniform FVector4 *uniform)&A);

	Result = REAL_BITS(intbits(S0) ^ SIGN_BIT);

	return *((uniform FVector4 *uniform)&Result);
}

inline uniform FVector VectorNegate(const uniform FVector &A)
{
	uniform FVector4 S0, Result;
	*((uniform FVector *uniform)&S0) = *((uniform FVector *uniform)&A);

	Result = VectorNegate(S0);

	return *((uniform FVector *uniform)&Result);
}

inline uniform FVector4 VectorMod(const uniform FVector4& X, const uniform FVector4& Y)
{
	const uniform FVector4 Div = VectorDivide(X, Y);
	// Floats where abs(f) >= 2^23 have no fractional portion, and larger values would overflow VectorTruncate.
	const uniform FVector4 NoFractionMask = VectorCompareGE(VectorAbs(Div), VectorNonFractional);
	const uniform FVector4 Temp = VectorSelect(NoFractionMask, Div, VectorTruncate(Div));
	const uniform FVector4 Result = VectorSubtract(X, VectorMultiply(Y, Temp));
	// Clamp to [-AbsY, AbsY] because of possible failures for very large numbers (>1e10) due to precision loss.
	const uniform FVector4 AbsY = VectorAbs(Y);
	return VectorMax(VectorNegate(AbsY), VectorMin(Result, AbsY));
}

/**
* Computes the sine and cosine of each component of a Vector.
*
* @param VSinAngles	FVector4 to where the Sin result should be stored
* @param VCosAngles	FVector4 to where the Cos result should be stored
* @param VAngles FVector4 to the input angles
*/
inline void VectorSinCos(uniform FVector4 &VSinAngles, uniform FVector4 &VCosAngles, const uniform FVector4 &VAngles)
{
	// Map to [-pi, pi]
	// X = A - 2pi * round(A/2pi)
	// Note the round(), not truncate(). In this case round() can round halfway cases using round-to-nearest-even OR round-to-nearest.

	// Quotient = round(A/2pi)
	uniform FVector4 Quotient = VectorMultiply(VAngles, VectorOneOverTwoPi);
	Quotient = VectorTruncate(Quotient); // round to nearest even is the default rounding mode but that's fine here.
	// X = A - 2pi * Quotient
	uniform FVector4 X = VectorSubtract(VAngles, VectorMultiply(VectorTwoPi, Quotient));

	// Map in [-pi/2,pi/2]
	uniform FVector4 sign = VectorBitwiseAnd(X, SIGN_BIT);
	const uniform FVector4 c = VectorBitwiseOr(VectorPi, sign);  // pi when x >= 0, -pi when x < 0
	const uniform FVector4 absx = VectorAbs(X);
	const uniform FVector4 rflx = VectorSubtract(c, X);
	const uniform FVector4 comp = VectorCompareGT(absx, VectorPiByTwo);
	X = VectorSelect(comp, rflx, X);
	sign = VectorSelect(comp, VectorMinusOne, VectorOne);

	const uniform FVector4 XSquared = VectorMultiply(X, X);

	// 11-degree minimax approximation
	//*ScalarSin = (((((-2.3889859e-08f * y2 + 2.7525562e-06f) * y2 - 0.00019840874f) * y2 + 0.0083333310f) * y2 - 0.16666667f) * y2 + 1.0f) * y;
	static const uniform FVector4d SinCoeff0 = {{ 1.0d, -0.16666667d, 0.0083333310d, -0.00019840874d }};
	static const uniform FVector4d SinCoeff1 = {{ 2.7525562d-06, -2.3889859d-08, /*unused*/ 0.d, /*unused*/ 0.d }};

	uniform FVector4 S;
	S = VectorReplicate(SinCoeff1, 1);
	S = VectorMultiplyAdd(XSquared, S, VectorReplicate(SinCoeff1, 0));
	S = VectorMultiplyAdd(XSquared, S, VectorReplicate(SinCoeff0, 3));
	S = VectorMultiplyAdd(XSquared, S, VectorReplicate(SinCoeff0, 2));
	S = VectorMultiplyAdd(XSquared, S, VectorReplicate(SinCoeff0, 1));
	S = VectorMultiplyAdd(XSquared, S, VectorReplicate(SinCoeff0, 0));
	VSinAngles = VectorMultiply(S, X);

	// 10-degree minimax approximation
	//*ScalarCos = sign * (((((-2.6051615e-07f * y2 + 2.4760495e-05f) * y2 - 0.0013888378f) * y2 + 0.041666638f) * y2 - 0.5f) * y2 + 1.0f);
	static const uniform FVector4d CosCoeff0 = {{ 1.0d, -0.5d, 0.041666638d, -0.0013888378d }};
	static const uniform FVector4d CosCoeff1 = {{ 2.4760495d-05, -2.6051615d-07, /*unused*/ 0.d, /*unused*/ 0.d }};

	uniform FVector4 C;
	C = VectorReplicate(CosCoeff1, 1);
	C = VectorMultiplyAdd(XSquared, C, VectorReplicate(CosCoeff1, 0));
	C = VectorMultiplyAdd(XSquared, C, VectorReplicate(CosCoeff0, 3));
	C = VectorMultiplyAdd(XSquared, C, VectorReplicate(CosCoeff0, 2));
	C = VectorMultiplyAdd(XSquared, C, VectorReplicate(CosCoeff0, 1));
	C = VectorMultiplyAdd(XSquared, C, VectorReplicate(CosCoeff0, 0));
	VCosAngles = VectorMultiply(C, sign);
}

template<>
inline FVector4d VectorLerp<FVector4d, double>(const FVector4d& A, const FVector4d& B, const double Alpha)
{
	const FVector4d Delta = VectorSubtract(B, A);
	return VectorMultiplyAdd(Alpha, Delta, A);
}

template<>
inline FVector4f VectorLerp<FVector4f, float>(const FVector4f& A, const FVector4f& B, const float Alpha)
{
	const FVector4f Delta = VectorSubtract(B, A);
	return VectorMultiplyAdd(Alpha, Delta, A);
}

template<>
inline uniform FVector4 VectorLerp<uniform FVector4, uniform FReal>(const uniform FVector4& A, const uniform FVector4& B, const uniform FReal Alpha)
{
	const uniform FVector4 Delta = VectorSubtract(B, A);
	return VectorMultiplyAdd(Alpha, Delta, A);
}

inline FReal VectorSizeSquared(const FVector4 &A)
{
	return VectorDot(A, A);
}

inline uniform FReal VectorSizeSquared(const uniform FVector4 &A)
{
	return VectorDot(A, A);
}

inline double VectorSizeSquared(const FVector3d &A)
{
	return VectorDot(A, A);
}

inline float VectorSizeSquared(const FVector3f &A)
{
	return VectorDot(A, A);
}

inline uniform double VectorSizeSquared(const uniform FVector3d &A)
{
	return VectorDot(A, A);
}

inline uniform float VectorSizeSquared(const uniform FVector3f &A)
{
	return VectorDot(A, A);
}

inline uniform double VectorSize(const uniform FVector3d& A)
{
	return sqrt(VectorSizeSquared(A));
}

inline uniform float VectorSize(const uniform FVector3f& A)
{
	return sqrt(VectorSizeSquared(A));
}

inline double VectorSize(const varying FVector3d &A)
{
	return sqrt(VectorDot(A, A));
}

inline float VectorSize(const varying FVector3f &A)
{
	return sqrt(VectorDot(A, A));
}

inline uniform FVector3d VectorGetSafeNormal(const uniform FVector3d &Vector)
{
	const uniform double SquareSum = VectorSizeSquared(Vector);

	// Not sure if it's safe to add tolerance in there. Might introduce too many errors
	if(SquareSum == 1.)
	{
		return Vector;
	}
	else if(SquareSum < DOUBLE_SMALL_NUMBER)
	{
		return ZeroVector;
	}
	const uniform double Scale = InvSqrt(SquareSum);
	return Vector * Scale;
}

inline uniform FVector3f VectorGetSafeNormal(const uniform FVector3f &Vector)
{
	const uniform float SquareSum = VectorSizeSquared(Vector);

	// Not sure if it's safe to add tolerance in there. Might introduce too many errors
	if(SquareSum == 1.f)
	{
		return Vector;
	}
	else if(SquareSum < FLOAT_SMALL_NUMBER)
	{
		return FloatZeroVector;
	}
	const uniform float Scale = InvSqrt(SquareSum);
	return Vector * Scale;
}

inline FVector4 VectorFloor(const FVector4 &A)
{
	return SetVector4(floor(A.V[0]), floor(A.V[1]), floor(A.V[2]), floor(A.V[3]));
}

template<>
inline FVector4 VectorClamp<FVector4, FVector4>(const FVector4 &A, const FVector4 &B, const FVector4 &C)
{
	return SetVector4(clamp(A.V[0], B.V[0], C.V[0]),
		clamp(A.V[1], B.V[1], C.V[1]),
		clamp(A.V[2], B.V[2], C.V[2]),
		clamp(A.V[3], B.V[3], C.V[3]));
}

template<>
inline FVector4 VectorClamp<FVector4, uniform FVector4>(const FVector4 &A, const uniform FVector4 &B, const uniform FVector4 &C)
{
	return SetVector4(clamp(A.V[0], B.V[0], C.V[0]),
		clamp(A.V[1], B.V[1], C.V[1]),
		clamp(A.V[2], B.V[2], C.V[2]),
		clamp(A.V[3], B.V[3], C.V[3]));
}

template<typename T>
inline uniform T VectorReduceAdd(const T& A)
{
	return SetVector(reduce_add(A.V[0]), reduce_add(A.V[1]), reduce_add(A.V[2]));
}

template<>
inline uniform FVector4 VectorReduceAdd<FVector4>(const FVector4& A)
{
	return SetVector4(reduce_add(A.V[0]), reduce_add(A.V[1]), reduce_add(A.V[2]), reduce_add(A.V[3]));
}

template <typename T>
inline uniform T VectorReduceMin(const T& A)
{
	return SetVector(reduce_min(A.V[0]), reduce_min(A.V[1]), reduce_min(A.V[2]));
}

template <typename T>
inline uniform T VectorReduceMax(const T& A)
{
	return SetVector(reduce_max(A.V[0]), reduce_max(A.V[1]), reduce_max(A.V[2]));
}

inline FVector3d VectorGetSafeNormal(const FVector3d &V, const uniform FVector3d &ResultIfZero)
{
	//We want N / ||N|| and to avoid inf
	//So we want N / ||N|| < 1 / eps => N eps < ||N||, but this is clearly true for all eps < 1 and N > 0
	double SizeSqr = VectorSizeSquared(V);
	return VectorSelect(SizeSqr <= DBL_MIN, ResultIfZero, V / sqrt(SizeSqr));
}

inline FVector3f VectorGetSafeNormal(const FVector3f &V, const uniform FVector3f &ResultIfZero)
{
	//We want N / ||N|| and to avoid inf
	//So we want N / ||N|| < 1 / eps => N eps < ||N||, but this is clearly true for all eps < 1 and N > 0
	float SizeSqr = VectorSizeSquared(V);
	return VectorSelect(SizeSqr <= FLT_MIN, ResultIfZero, V / sqrt(SizeSqr));
}

inline FVector3d VectorGetSafeNormal(const FVector3d &V)
{
	return VectorGetSafeNormal(V, DoubleForwardVector);
}

inline FVector3f VectorGetSafeNormal(const FVector3f &V)
{
	return VectorGetSafeNormal(V, FloatForwardVector);
}

#endif
