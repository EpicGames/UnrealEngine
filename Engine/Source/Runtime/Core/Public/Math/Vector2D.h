// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/MathFwd.h" // IWYU pragma: export
#include "Misc/AssertionMacros.h"
#include "Misc/Crc.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Misc/Parse.h"
#include "Misc/LargeWorldCoordinatesSerializer.h"
#include "Misc/EngineNetworkCustomVersion.h"
#include "Math/IntPoint.h"
#include "Logging/LogMacros.h"

#include <type_traits>

#ifdef _MSC_VER
#pragma warning (push)
// Ensure template functions don't generate shadowing warnings against global variables at the point of instantiation.
#pragma warning (disable : 4459)
#endif


/**
 * A vector in 2-D space composed of components (X, Y) with floating point precision.
 */
namespace UE
{
namespace Math
{
template<typename T>	
struct TVector2 
{
	static_assert(std::is_floating_point_v<T>, "T must be floating point");

public:
	using FReal = T;
	
	union
	{
		struct
		{
			/** Vector's X component. */
			T X;

			/** Vector's Y component. */
			T Y;
		};

		UE_DEPRECATED(all, "For internal use only")
		T XY[2];
	};

	/** The number of components this vector type has. */
	static constexpr int32 NumComponents = 2;

	/** Global 2D zero vector constant (0,0) */
	CORE_API static const TVector2<T> ZeroVector;

	/**
	* Global 2D one vector (poorly named) constant (1,1).
	*
	* @note Incorrectly named "unit" vector though its magnitude/length/size is not one. Would fix, though likely used all over the world. Use `Unit45Deg` below for an actual unit vector.
	*/
	CORE_API static const TVector2<T> UnitVector;

	/**
	* Global 2D unit vector constant along the 45 degree angle or symmetrical positive axes (sqrt(.5),sqrt(.5)) or (.707,.707). https://en.wikipedia.org/wiki/Unit_vector
	*
	* @note The `UnitVector` above is actually a value with axes of 1 rather than a magnitude of one.
	*/
	CORE_API static const TVector2<T> Unit45Deg;

	static inline TVector2<T> Zero() { return TVector2<T>::ZeroVector; }
	static inline TVector2<T> One() { return TVector2<T>::UnitVector; }
	static inline TVector2<T> UnitX() { return TVector2<T>(1, 0); }
	static inline TVector2<T> UnitY() { return TVector2<T>(0, 1); }

public:

	/** Default constructor (no initialization). */
	[[nodiscard]] TVector2() = default;

	/**
	* Constructor using initial values for each component.
	*
	* @param InX X coordinate.
	* @param InY Y coordinate.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TVector2<T>(T InX, T InY);

	/**
	* Constructor initializing both components to a single T value.
	*
	* @param InF Value to set both components to.
	*/
	[[nodiscard]] explicit UE_FORCEINLINE_HINT TVector2<T>(T InF);

	/**
	* Constructs a vector from an FIntPoint.
	*
	* @param InPos Integer point used to set this vector.
	*/
	template <typename IntType>
	[[nodiscard]] UE_FORCEINLINE_HINT TVector2<T>(TIntPoint<IntType> InPos);

	template <typename IntType>
    [[nodiscard]] explicit  TVector2<T>(TIntVector2<IntType> V);

	/**
	* Constructor which initializes all components to zero.
	*
	* @param EForceInit Force init enum
	*/
	[[nodiscard]] explicit UE_FORCEINLINE_HINT TVector2<T>(EForceInit);

	/**
	* Constructor that does not initialize.  More explicit than the default constructor.
	*
	* @param ENoInit Don't init
	*/
	[[nodiscard]] explicit UE_FORCEINLINE_HINT TVector2<T>(ENoInit) { }

	/**
	* Constructs a vector from an FVector.
	* Copies the X and Y components from the FVector.
	*
	* @param V Vector to copy from.
	*/
	[[nodiscard]] explicit inline TVector2<T>(const TVector<T>& V);

	/**
	* Constructs a vector from an FVector4.
	* Copies the X and Y components from the FVector4.
	*
	* @param V Vector to copy from.
	*/
	[[nodiscard]] explicit inline TVector2<T>(const TVector4<T>& V);

public:

	/**
	* Gets the result of adding two vectors together.
	*
	* @param V The other vector to add to this.
	* @return The result of adding the vectors together.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TVector2<T> operator+(const TVector2<T>& V) const;

	/**
	* Gets the result of subtracting a vector from this one.
	*
	* @param V The other vector to subtract from this.
	* @return The result of the subtraction.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TVector2<T> operator-(const TVector2<T>& V) const;

	/**
	* Gets the result of scaling the vector (multiplying each component by a value).
	*
	* @param Scale How much to scale the vector by.
	* @return The result of scaling this vector.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TVector2<T> operator*(T Scale) const;

	/**
	* Gets the result of dividing each component of the vector by a value.
	*
	* @param Scale How much to divide the vector by.
	* @return The result of division on this vector.
	*/
	[[nodiscard]] TVector2<T> operator/(T Scale) const;

	/**
	* Gets the result of adding A to each component of the vector.
	*
	* @param A T to add to each component.
	* @return The result of adding A to each component.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TVector2<T> operator+(T A) const;

	/**
	* Gets the result of subtracting A from each component of the vector.
	*
	* @param A T to subtract from each component
	* @return The result of subtracting A from each component.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TVector2<T> operator-(T A) const;

	/**
	* Gets the result of component-wise multiplication of this vector by another.
	*
	* @param V The other vector to multiply this by.
	* @return The result of the multiplication.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TVector2<T> operator*(const TVector2<T>& V) const;

	/**
	* Gets the result of component-wise division of this vector by another.
	*
	* @param V The other vector to divide this by.
	* @return The result of the division.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TVector2<T> operator/(const TVector2<T>& V) const;

	/**
	* Calculates dot product of this vector and another.
	*
	* @param V The other vector.
	* @return The dot product.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT T operator|(const TVector2<T>& V) const;

	/**
	* Calculates cross product of this vector and another.
	*
	* @param V The other vector.
	* @return The cross product.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT T operator^(const TVector2<T>& V) const;

public:

	/**
	* Compares this vector against another for equality.
	*
	* @param V The vector to compare against.
	* @return true if the two vectors are equal, otherwise false.
	*/
	[[nodiscard]] bool operator==(const TVector2<T>& V) const;

	/**
	* Compares this vector against another for inequality.
	*
	* @param V The vector to compare against.
	* @return true if the two vectors are not equal, otherwise false.
	*/
	[[nodiscard]] bool operator!=(const TVector2<T>& V) const;

	/**
	* Checks whether both components of this vector are less than another.
	*
	* @param Other The vector to compare against.
	* @return true if both components of this are less than Other, otherwise false.
	*/
	[[nodiscard]] bool ComponentwiseAllLessThan(const TVector2<T>& Other) const;

	/**
	* Checks whether both components of this vector are greater than another.
	*
	* @param Other The vector to compare against.
	* @return true if both components of this are greater than Other, otherwise false.
	*/
	[[nodiscard]] bool ComponentwiseAllGreaterThan(const TVector2<T>& Other) const;

	/**
	* Checks whether both components of this vector are less than or equal to another.
	*
	* @param Other The vector to compare against.
	* @return true if both components of this are less than or equal to Other, otherwise false.
	*/
	[[nodiscard]] bool ComponentwiseAllLessOrEqual(const TVector2<T>& Other) const;

	/**
	* Checks whether both components of this vector are greater than or equal to another.
	*
	* @param Other The vector to compare against.
	* @return true if both components of this are greater than or equal to Other, otherwise false.
	*/
	[[nodiscard]] bool ComponentwiseAllGreaterOrEqual(const TVector2<T>& Other) const;


	/**
	* Gets a negated copy of the vector.
	*
	* @return A negated copy of the vector.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TVector2<T> operator-() const;

	/**
	* Adds another vector to this.
	*
	* @param V The other vector to add.
	* @return Copy of the vector after addition.
	*/
	inline TVector2<T> operator+=(const TVector2<T>& V);

	/**
	* Subtracts another vector from this.
	*
	* @param V The other vector to subtract.
	* @return Copy of the vector after subtraction.
	*/
	inline TVector2<T> operator-=(const TVector2<T>& V);

	/**
	* Scales this vector.
	*
	* @param Scale The scale to multiply vector by.
	* @return Copy of the vector after scaling.
	*/
	inline TVector2<T> operator*=(T Scale);

	/**
	* Divides this vector.
	*
	* @param V What to divide vector by.
	* @return Copy of the vector after division.
	*/
	TVector2<T> operator/=(T V);

	/**
	* Multiplies this vector with another vector, using component-wise multiplication.
	*
	* @param V The vector to multiply with.
	* @return Copy of the vector after multiplication.
	*/
	TVector2<T> operator*=(const TVector2<T>& V);

	/**
	* Divides this vector by another vector, using component-wise division.
	*
	* @param V The vector to divide by.
	* @return Copy of the vector after division.
	*/
	TVector2<T> operator/=(const TVector2<T>& V);

	/**
	* Gets specific component of the vector.
	*
	* @param Index the index of vector component
	* @return reference to component.
	*/
	[[nodiscard]] T& operator[](int32 Index);

	/**
	* Gets specific component of the vector.
	*
	* @param Index the index of vector component
	* @return copy of component value.
	*/
	[[nodiscard]] T operator[](int32 Index) const;

	/**
	* Gets a specific component of the vector.
	*
	* @param Index The index of the component required.
	* @return Reference to the specified component.
	*/
	[[nodiscard]] T& Component(int32 Index);

	/**
	* Gets a specific component of the vector.
	*
	* @param Index The index of the component required.
	* @return Copy of the specified component.
	*/
	[[nodiscard]] T Component(int32 Index) const;

	/**
	 * Tests if index is valid, i.e. greater than or equal to zero, and less than the number of components in the vector.
	 *
	 * @param Index Index to test.
	 * @returns True if index is valid. False otherwise.
	 */
	[[nodiscard]] bool IsValidIndex(int32 Index) const;

public:

	/**
	* Calculates the dot product of two vectors.
	*
	* @param A The first vector.
	* @param B The second vector.
	* @return The dot product.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT static T DotProduct(const TVector2<T>& A, const TVector2<T>& B);

	/**
	* Squared distance between two 2D points.
	*
	* @param V1 The first point.
	* @param V2 The second point.
	* @return The squared distance between two 2D points.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT static T DistSquared(const TVector2<T>& V1, const TVector2<T>& V2);

	/**
	* Distance between two 2D points.
	*
	* @param V1 The first point.
	* @param V2 The second point.
	* @return The distance between two 2D points.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT static T Distance(const TVector2<T>& V1, const TVector2<T>& V2);

	/**
	* Calculate the cross product of two vectors.
	*
	* @param A The first vector.
	* @param B The second vector.
	* @return The cross product.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT static T CrossProduct(const TVector2<T>& A, const TVector2<T>& B);

	/**
	* Returns a vector with the maximum component for each dimension from the pair of vectors.
	*
	* @param A The first vector.
	* @param B The second vector.
	* @return The max vector.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT static TVector2<T> Max(const TVector2<T>& A, const TVector2<T>& B);

	/**
	* Returns a vector with the minimum component for each dimension from the pair of vectors.
	*
	* @param A The first vector.
	* @param B The second vector.
	* @return The min vector.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT static TVector2<T> Min(const TVector2<T>& A, const TVector2<T>& B);

	/**
	* Returns a vector with each component clamped between a minimum and a maximum.
	*
	* @param V The vector to clamp.
	* @param MinValue The minimum vector.
	* @param MaxValue The maximum vector.
	* @return The clamped vector.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT static TVector2<T> Clamp(const TVector2<T>& V, const TVector2<T>& MinValue, const TVector2<T>& MaxValue);

	/**
	* Checks for equality with error-tolerant comparison.
	*
	* @param V The vector to compare.
	* @param Tolerance Error tolerance.
	* @return true if the vectors are equal within specified tolerance, otherwise false.
	*/
	[[nodiscard]] bool Equals(const TVector2<T>& V, T Tolerance=UE_KINDA_SMALL_NUMBER) const;

	/**
	* Set the values of the vector directly.
	*
	* @param InX New X coordinate.
	* @param InY New Y coordinate.
	*/
	void Set(T InX, T InY);

	/**
	* Get the maximum value of the vector's components.
	*
	* @return The maximum value of the vector's components.
	*/
	[[nodiscard]] T GetMax() const;

	/**
	* Get the maximum absolute value of the vector's components.
	*
	* @return The maximum absolute value of the vector's components.
	*/
	[[nodiscard]] T GetAbsMax() const;

	/**
	* Get the minimum value of the vector's components.
	*
	* @return The minimum value of the vector's components.
	*/
	[[nodiscard]] T GetMin() const;

	/** Gets the component-wise min of two vectors. */
	[[nodiscard]] TVector2<T> ComponentMin(const TVector2<T>& Other) const;

	/** Gets the component-wise max of two vectors. */
	[[nodiscard]] TVector2<T> ComponentMax(const TVector2<T>& Other) const;

	/**
	* Get the length (magnitude) of this vector.
	*
	* @return The length of this vector.
    * @see Length - This function is a synonym for Length()
	*/
	[[nodiscard]] T Size() const;

	/**
    * Get the length (magnitude) of this vector.
    *
    * @return The length of this vector.
    * @see Size - This function is a synonym for Size()
    */
	[[nodiscard]] UE_FORCEINLINE_HINT T Length() const
	{
		return Size();
	}

	/**
	* Get the squared length of this vector.
	*
	* @return The squared length of this vector.
    * @see LengthSquared - This function is a synonym for LengthSquared()
	*/
	[[nodiscard]] T SizeSquared() const;
	
	/**
	* Get the squared length of this vector.
	*
	* @return The squared length of this vector.
    * @see SizeSquared - This function is a synonym for SizeSquared()
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT T SquaredLength() const
	{
		return SizeSquared();
	}

	/**
	* Get the dot product of this vector against another.
	*
	* @param V2 The vector to measure dot product against.
	* @return The dot product.
	*/
	[[nodiscard]] T Dot(const TVector2<T>& V2) const;

	/**
	* Rotates around axis (0,0,1)
	*
	* @param AngleDeg Angle to rotate (in degrees)
	* @return Rotated Vector
	*/
	[[nodiscard]] TVector2<T> GetRotated(T AngleDeg) const;

	/**
	* Gets a normalized copy of the vector, checking it is safe to do so based on the length.
	* Returns zero vector if vector length is too small to safely normalize.
	*
	* @param Tolerance Minimum squared length of vector for normalization.
	* @return A normalized copy of the vector if safe, (0,0) otherwise.
	*/
	[[nodiscard]] TVector2<T> GetSafeNormal(T Tolerance=UE_SMALL_NUMBER) const;

	/**
	* Normalize this vector in-place if it is large enough, set it to (0,0) otherwise.
	* (Note this is different from TVector<>::Normalize, which leaves the vector unchanged if it is too small to normalize.)
	*
	* @param Tolerance Minimum squared length of vector for normalization.
	* @see GetSafeNormal()
	* @return true if the vector was normalized correctly, false if it was too small and set to zero.
	*/
	bool Normalize(T Tolerance=UE_SMALL_NUMBER);

	/**
	* Checks whether vector is near to zero within a specified tolerance.
	*
	* @param Tolerance Error tolerance.
	* @return true if vector is in tolerance to zero, otherwise false.
	*/
	[[nodiscard]] bool IsNearlyZero(T Tolerance=UE_KINDA_SMALL_NUMBER) const;

	/**
	* Util to convert this vector into a unit direction vector and its original length.
	*
	* @param OutDir Reference passed in to store unit direction vector.
	* @param OutLength Reference passed in to store length of the vector.
	*/
	void ToDirectionAndLength(TVector2<T>& OutDir, T& OutLength) const;
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	void ToDirectionAndLength(TVector2<T>& OutDir, FArg& OutLength) const
	{
		FReal TempOut;
		ToDirectionAndLength(OutDir, TempOut);
		OutLength = static_cast<FArg>(TempOut);
	}

	/**
	* Checks whether all components of the vector are exactly zero.
	*
	* @return true if vector is exactly zero, otherwise false.
	*/
	[[nodiscard]] bool IsZero() const;

	/**
	* Get this vector as an Int Point.
	*
	* @return New Int Point from this vector.
	*/
	[[nodiscard]] FIntPoint IntPoint() const;

	/**
	* Get this vector as a vector where each component has been rounded to the nearest int.
	*
	* @return New TVector2<T> from this vector that is rounded.
	*/
	[[nodiscard]] TVector2<T> RoundToVector() const;

	/**
	* Creates a copy of this vector with both axes clamped to the given range.
	* @return New vector with clamped axes.
	*/
	[[nodiscard]] TVector2<T> ClampAxes(T MinAxisVal, T MaxAxisVal) const;

	/**
	* Create a copy of this vector, with its magnitude clamped between Min and Max.
	* @return New vector with clamped magnitude.
	*/
	[[nodiscard]] TVector2<T> GetClampedToSize(T Min, T Max) const;

	/**
	* Create a copy of this vector, with its maximum magnitude clamped to MaxSize.
	* @return New vector with clamped magnitude.
	*/
	[[nodiscard]] TVector2<T> GetClampedToMaxSize(T MaxSize) const;

	/**
	* Get a copy of the vector as sign only.
	* Each component is set to +1 or -1, with the sign of zero treated as +1.
	*
	* @param A copy of the vector with each component set to +1 or -1
	*/
	[[nodiscard]] inline TVector2<T> GetSignVector() const;

	/**
	* Get a copy of this vector with absolute value of each component.
	*
	* @return A copy of this vector with absolute value of each component.
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TVector2<T> GetAbs() const;

	/**
	* Get a textual representation of the vector.
	*
	* @return Text describing the vector.
	*/
	[[nodiscard]] FString ToString() const;

	/**
	* Initialize this Vector based on an FString. The String is expected to contain X=, Y=.
	* The TVector2<T> will be bogus when InitFromString returns false.
	*
	* @param	InSourceString	FString containing the vector values.
	* @return true if the X,Y values were read successfully; false otherwise.
	*/
	bool InitFromString(const FString& InSourceString);

	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	bool Serialize(FStructuredArchive::FSlot Slot)
	{
		Slot << *this;
		return true;
	}

	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		if constexpr (std::is_same_v<T, float>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Vector2D, Vector2f, Vector2d);
		}
		else
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Vector2D, Vector2d, Vector2f);
		}
	}

#if ENABLE_NAN_DIAGNOSTIC
	inline void DiagnosticCheckNaN()
	{
		if (ContainsNaN())
		{
			logOrEnsureNanError(TEXT("FVector2 contains NaN: %s"), *ToString());
		}
	}
#else
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN() {}
#endif

	/**
	* Utility to check if there are any non-finite values (NaN or Inf) in this vector.
	*
	* @return true if there are any non-finite values in this vector, false otherwise.
	*/
	[[nodiscard]] inline bool ContainsNaN() const
	{
		return (!FMath::IsFinite(X) || 
				!FMath::IsFinite(Y));
	}

	/**
	* Network serialization function.
	* FVectors NetSerialize without quantization (ie exact values are serialized).
	*/
	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
	{
		Ar.UsingCustomVersion(FEngineNetworkCustomVersion::Guid);

		if (Ar.EngineNetVer() >= FEngineNetworkCustomVersion::SerializeDoubleVectorsAsDoubles && Ar.EngineNetVer() != FEngineNetworkCustomVersion::Ver21AndViewPitchOnly_DONOTUSE)
		{
			Ar << X << Y;
		}
		else
		{
			checkf(Ar.IsLoading(), TEXT("float -> double conversion applied outside of load!"));
			// Always serialize as float
			float SX, SY;
			Ar << SX << SY;
			X = SX;
			Y = SY;
		}
		return true;
	}

	/** Converts spherical coordinates on the unit sphere into a Cartesian unit length vector. */
	[[nodiscard]] inline TVector<T> SphericalToUnitCartesian() const;
	
	
	
	// Conversion from other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TVector2(const TVector2<FArg>& From)
		: TVector2<T>((T)From.X, (T)From.Y)
	{
	}
};

/**
* Creates a hash value from a TVector2<T>. 
*
* @param Vector the vector to create a hash value for
* @return The hash value from the components
*/
template<typename T>
UE_FORCEINLINE_HINT uint32 GetTypeHash(const TVector2<T>& Vector)
{
	// Note: this assumes there's no padding in TVector2<T> that could contain uncompared data.
	return FCrc::MemCrc_DEPRECATED(&Vector,sizeof(Vector));
}

/* TVector2<T> inline functions
*****************************************************************************/

/**
* Serialize a vector.
*
* @param Ar Serialization archive.
* @param V Vector being serialized.
* @return Reference to Archive after serialization.
*/
inline FArchive& operator<<(FArchive& Ar, TVector2<float>& V)
{
	// @warning BulkSerialize: TVector2<T> is serialized as memory dump
	// See TArray::BulkSerialize for detailed description of implied limitations.
	return Ar << V.X << V.Y;
}

inline void operator<<(FStructuredArchive::FSlot Slot, TVector2<float>& V)
{
	// @warning BulkSerialize: TVector2<T> is serialized as memory dump
	// See TArray::BulkSerialize for detailed description of implied limitations.
	FStructuredArchive::FStream Stream = Slot.EnterStream();
	Stream.EnterElement() << V.X;
	Stream.EnterElement() << V.Y;
}

inline FArchive& operator<<(FArchive& Ar, TVector2<double>& V)
{
	// @warning BulkSerialize: TVector2<T> is serialized as memory dump
	// See TArray::BulkSerialize for detailed description of implied limitations.
	if (Ar.UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
	{
		Ar << V.X << V.Y;
	}
	else
	{
		checkf(Ar.IsLoading(), TEXT("float -> double conversion applied outside of load!"));
		// Stored as floats, so serialize float and copy.
		float X, Y;
		Ar << X << Y;
		V = TVector2<double>(X, Y);
	}
	return Ar;
}

inline void operator<<(FStructuredArchive::FSlot Slot, TVector2<double>& V)
{
	// @warning BulkSerialize: TVector2<T> is serialized as memory dump
	// See TArray::BulkSerialize for detailed description of implied limitations.
	FStructuredArchive::FStream Stream = Slot.EnterStream();
	if (Slot.GetUnderlyingArchive().UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
	{
		Stream.EnterElement() << V.X;
		Stream.EnterElement() << V.Y;
	}
	else
	{
		checkf(Slot.GetUnderlyingArchive().IsLoading(), TEXT("float -> double conversion applied outside of load!"));
		// Stored as floats, so serialize float and copy.
		float X, Y;
		Stream.EnterElement() << X;
		Stream.EnterElement() << Y;
		V = TVector2<double>(X, Y);
	}
}

#if !defined(_MSC_VER) || defined(__clang__)  // MSVC can't forward declare explicit specializations
template<> CORE_API const FVector2f FVector2f::ZeroVector;
template<> CORE_API const FVector2f FVector2f::UnitVector;
template<> CORE_API const FVector2f FVector2f::Unit45Deg;
template<> CORE_API const FVector2d FVector2d::ZeroVector;
template<> CORE_API const FVector2d FVector2d::UnitVector;
template<> CORE_API const FVector2d FVector2d::Unit45Deg;
#endif

/**
 * Multiplies a Vector2 by a scaling factor.
 *
 * @param Scale Scaling factor.
 * @param V Vector2 to scale.
 * @return Result of multiplication.
 */
template<typename T, typename T2 UE_REQUIRES(std::is_arithmetic_v<T2>)>
UE_FORCEINLINE_HINT TVector2<T> operator*(T2 Scale, const TVector2<T>& V)
{
	return V.operator*(Scale);
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T>::TVector2(T InX,T InY)
	:	X(InX), Y(InY)
{ }

template<typename T>
UE_FORCEINLINE_HINT TVector2<T>::TVector2(T InF)
	:	X(InF), Y(InF)
{ }

template<typename T>
template<typename IntType>
inline TVector2<T>::TVector2(TIntPoint<IntType> InPos)
{
	X = (T)InPos.X;
	Y = (T)InPos.Y;
}

template<typename T>
template<typename IntType>
inline TVector2<T>::TVector2(TIntVector2<IntType> V)
{
	X = (T)V.X;
	Y = (T)V.Y;
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T>::TVector2(EForceInit)
	: X(0), Y(0)
{
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T> TVector2<T>::operator+(const TVector2<T>& V) const
{
	return TVector2<T>(X + V.X, Y + V.Y);
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T> TVector2<T>::operator-(const TVector2<T>& V) const
{
	return TVector2<T>(X - V.X, Y - V.Y);
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T> TVector2<T>::operator*(T Scale) const
{
	return TVector2<T>(X * Scale, Y * Scale);
}

template<typename T>
inline TVector2<T> TVector2<T>::operator/(T Scale) const
{
	const T RScale = 1.f/Scale;
	return TVector2<T>(X * RScale, Y * RScale);
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T> TVector2<T>::operator+(T A) const
{
	return TVector2<T>(X + A, Y + A);
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T> TVector2<T>::operator-(T A) const
{
	return TVector2<T>(X - A, Y - A);
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T> TVector2<T>::operator*(const TVector2<T>& V) const
{
	return TVector2<T>(X * V.X, Y * V.Y);
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T> TVector2<T>::operator/(const TVector2<T>& V) const
{
	return TVector2<T>(X / V.X, Y / V.Y);
}

template<typename T>
UE_FORCEINLINE_HINT T TVector2<T>::operator|(const TVector2<T>& V) const
{
	return X*V.X + Y*V.Y;
}

template<typename T>
UE_FORCEINLINE_HINT T TVector2<T>::operator^(const TVector2<T>& V) const
{
	return X*V.Y - Y*V.X;
}

template<typename T>
UE_FORCEINLINE_HINT T TVector2<T>::DotProduct(const TVector2<T>& A, const TVector2<T>& B)
{
	return A | B;
}

template<typename T>
UE_FORCEINLINE_HINT T TVector2<T>::DistSquared(const TVector2<T> &V1, const TVector2<T> &V2)
{
	return FMath::Square(V2.X-V1.X) + FMath::Square(V2.Y-V1.Y);
}

template<typename T>
UE_FORCEINLINE_HINT T TVector2<T>::Distance(const TVector2<T>& V1, const TVector2<T>& V2)
{
	return FMath::Sqrt(TVector2<T>::DistSquared(V1, V2));
}

template<typename T>
UE_FORCEINLINE_HINT T TVector2<T>::CrossProduct(const TVector2<T>& A, const TVector2<T>& B)
{
	return A ^ B;
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T> TVector2<T>::Max(const TVector2<T>& A, const TVector2<T>& B)
{
	return TVector2<T>(FMath::Max(A.X, B.X), FMath::Max(A.Y, B.Y));
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T> TVector2<T>::Min(const TVector2<T>& A, const TVector2<T>& B)
{
	return TVector2<T>(FMath::Min(A.X, B.X), FMath::Min(A.Y, B.Y));
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T> TVector2<T>::Clamp(const TVector2<T>& V, const TVector2<T>& MinValue, const TVector2<T>& MaxValue)
{
	return TVector2<T>(FMath::Clamp(V.X, MinValue.X, MaxValue.X), FMath::Clamp(V.Y, MinValue.Y, MaxValue.Y));
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector2<T>::operator==(const TVector2<T>& V) const
{
	return X==V.X && Y==V.Y;
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector2<T>::operator!=(const TVector2<T>& V) const
{
	return X!=V.X || Y!=V.Y;
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector2<T>::ComponentwiseAllLessThan(const TVector2<T>& Other) const
{
	return X < Other.X && Y < Other.Y;
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector2<T>::ComponentwiseAllGreaterThan(const TVector2<T>& Other) const
{
	return X > Other.X && Y > Other.Y;
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector2<T>::ComponentwiseAllLessOrEqual(const TVector2<T>& Other) const
{
	return X <= Other.X && Y <= Other.Y;
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector2<T>::ComponentwiseAllGreaterOrEqual(const TVector2<T>& Other) const
{
	return X >= Other.X && Y >= Other.Y;
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector2<T>::Equals(const TVector2<T>& V, T Tolerance) const
{
	return FMath::Abs(X-V.X) <= Tolerance && FMath::Abs(Y-V.Y) <= Tolerance;
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T> TVector2<T>::operator-() const
{
	return TVector2<T>(-X, -Y);
}

template<typename T>
inline TVector2<T> TVector2<T>::operator+=(const TVector2<T>& V)
{
	X += V.X; Y += V.Y;
	return *this;
}

template<typename T>
inline TVector2<T> TVector2<T>::operator-=(const TVector2<T>& V)
{
	X -= V.X; Y -= V.Y;
	return *this;
}

template<typename T>
inline TVector2<T> TVector2<T>::operator*=(T Scale)
{
	X *= Scale; Y *= Scale;
	return *this;
}

template<typename T>
inline TVector2<T> TVector2<T>::operator/=(T V)
{
	const T RV = 1.f/V;
	X *= RV; Y *= RV;
	return *this;
}

template<typename T>
inline TVector2<T> TVector2<T>::operator*=(const TVector2<T>& V)
{
	X *= V.X; Y *= V.Y;
	return *this;
}

template<typename T>
inline TVector2<T> TVector2<T>::operator/=(const TVector2<T>& V)
{
	X /= V.X; Y /= V.Y;
	return *this;
}

template<typename T>
UE_FORCEINLINE_HINT T& TVector2<T>::operator[](int32 Index)
{
	return Component(Index);
}

template<typename T>
UE_FORCEINLINE_HINT T TVector2<T>::operator[](int32 Index) const
{
	return Component(Index);
}

template<typename T>
inline void TVector2<T>::Set(T InX, T InY)
{
	X = InX;
	Y = InY;
}

template<typename T>
UE_FORCEINLINE_HINT T TVector2<T>::GetMax() const
{
	return FMath::Max(X,Y);
}

template<typename T>
UE_FORCEINLINE_HINT T TVector2<T>::GetAbsMax() const
{
	return FMath::Max(FMath::Abs(X),FMath::Abs(Y));
}

template<typename T>
UE_FORCEINLINE_HINT T TVector2<T>::GetMin() const
{
	return FMath::Min(X,Y);
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T> TVector2<T>::ComponentMin(const TVector2<T>& Other) const
{
    return TVector2<T>(FMath::Min(X, Other.X), FMath::Min(Y, Other.Y));
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T> TVector2<T>::ComponentMax(const TVector2<T>& Other) const
{
    return TVector2<T>(FMath::Max(X, Other.X), FMath::Max(Y, Other.Y));
}

template<typename T>
UE_FORCEINLINE_HINT T TVector2<T>::Size() const
{
	return FMath::Sqrt(X*X + Y*Y);
}

template<typename T>
UE_FORCEINLINE_HINT T TVector2<T>::SizeSquared() const
{
	return X*X + Y*Y;
}

template<typename T>
UE_FORCEINLINE_HINT T TVector2<T>::Dot(const TVector2<T>& V2) const
{ 
	return X * V2.X + Y * V2.Y;
} 

template<typename T>
inline TVector2<T> TVector2<T>::GetRotated(const T AngleDeg) const
{
	// Based on FVector::RotateAngleAxis with Axis(0,0,1)

	T S, C;
	FMath::SinCos(&S, &C, FMath::DegreesToRadians(AngleDeg));

	return TVector2<T>(
		C * X - S * Y,
		S * X + C * Y);
}

template<typename T>
inline TVector2<T> TVector2<T>::GetSafeNormal(T Tolerance) const
{	
	const T SquareSum = X*X + Y*Y;
	if(SquareSum > Tolerance)
	{
		const T Scale = FMath::InvSqrt(SquareSum);
		return TVector2<T>(X*Scale, Y*Scale);
	}
	return TVector2<T>(0.f, 0.f);
}

template<typename T>
inline bool TVector2<T>::Normalize(T Tolerance)
{
	const T SquareSum = X*X + Y*Y;
	if(SquareSum > Tolerance)
	{
		const T Scale = FMath::InvSqrt(SquareSum);
		X *= Scale;
		Y *= Scale;
		return true;
	}
	X = 0.0f;
	Y = 0.0f;
	return false;
}

template<typename T>
inline void TVector2<T>::ToDirectionAndLength(TVector2<T> &OutDir, T &OutLength) const
{
	OutLength = Size();
	if (OutLength > UE_SMALL_NUMBER)
	{
		T OneOverLength = static_cast<T>(1.0f) / OutLength;
		OutDir = TVector2<T>(static_cast<T>(X*OneOverLength), static_cast<T>(Y*OneOverLength));
	}
	else
	{
		OutDir = TVector2<T>::ZeroVector;
	}
}

template<typename T>
inline bool TVector2<T>::IsNearlyZero(T Tolerance) const
{
	return	FMath::Abs(X)<=Tolerance
		&&	FMath::Abs(Y)<=Tolerance;
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector2<T>::IsZero() const
{
	return X==0.f && Y==0.f;
}

template<typename T>
inline T& TVector2<T>::Component(int32 Index)
{
	check(IsValidIndex(Index));
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return XY[Index];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

template<typename T>
inline T TVector2<T>::Component(int32 Index) const
{
	check(IsValidIndex(Index));
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return XY[Index];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

template<typename T>
bool TVector2<T>::IsValidIndex(int32 Index) const
{
	return (Index >= 0) && (Index < NumComponents);
}

template<typename T>
inline FIntPoint TVector2<T>::IntPoint() const
{
	if constexpr (std::is_same_v<T, float>)
	{
		return FIntPoint( FMath::RoundToInt32(X), FMath::RoundToInt32(Y) );
	}
	else
	{
		// FIntPoint constructor from FInt64Point checks that the int64 fits in int32.
		return FIntPoint( FInt64Point(FMath::RoundToInt64(X), FMath::RoundToInt64(Y)) );
	}
}

template<typename T>
inline TVector2<T> TVector2<T>::RoundToVector() const
{
	if constexpr (std::is_same_v<T, float>)
	{
		return TVector2<T>(FMath::RoundToFloat(X), FMath::RoundToFloat(Y));
	}
	else
	{
		return TVector2<T>(FMath::RoundToDouble(X), FMath::RoundToDouble(Y));
	}
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T> TVector2<T>::ClampAxes(T MinAxisVal, T MaxAxisVal) const
{
	return TVector2<T>(FMath::Clamp(X, MinAxisVal, MaxAxisVal), FMath::Clamp(Y, MinAxisVal, MaxAxisVal));
}

template<typename T>
inline TVector2<T> TVector2<T>::GetClampedToSize(T Min, T Max) const
{
	T VecSize = Size();
	const TVector2<T> VecDir = (VecSize > UE_SMALL_NUMBER) ? (*this / VecSize) : ZeroVector;

	VecSize = FMath::Clamp(VecSize, Min, Max);

	return VecSize * VecDir;
}

template<typename T>
inline TVector2<T> TVector2<T>::GetClampedToMaxSize(T MaxSize) const
{
	if (MaxSize < UE_KINDA_SMALL_NUMBER)
	{
		return ZeroVector;
	}

	const T VSq = SizeSquared();
	if (VSq > FMath::Square(MaxSize))
	{
		const T Scale = MaxSize * FMath::InvSqrt(VSq);
		return TVector2<T>(X * Scale, Y * Scale);
	}
	else
	{
		return *this;
	}
}

template<typename T>
inline TVector2<T> TVector2<T>::GetSignVector() const
{
	return TVector2<T>
		(
		FMath::FloatSelect(X, (T)1, (T)-1),
		FMath::FloatSelect(Y, (T)1, (T)-1)
		);
}

template<typename T>
UE_FORCEINLINE_HINT TVector2<T> TVector2<T>::GetAbs() const
{
	return TVector2<T>(FMath::Abs(X), FMath::Abs(Y));
}

template<typename T>
UE_FORCEINLINE_HINT FString TVector2<T>::ToString() const
{
	return FString::Printf(TEXT("X=%3.3f Y=%3.3f"), X, Y);
}

template<typename T>
inline bool TVector2<T>::InitFromString(const FString& InSourceString)
{
	X = Y = 0;

	// The initialization is only successful if the X and Y values can all be parsed from the string
	const bool bSuccessful = FParse::Value(*InSourceString, TEXT("X=") , X) && FParse::Value(*InSourceString, TEXT("Y="), Y) ;

	return bSuccessful;
}

} // namespace UE::Math
} // UE

UE_DECLARE_LWC_TYPE(Vector2,, FVector2D);

template <> struct TIsPODType<FVector2f> { enum { Value = true }; };
template <> struct TIsUECoreVariant<FVector2f> { enum { Value = true }; };
template<> struct TCanBulkSerialize<FVector2f> { enum { Value = true }; };
template <> struct TIsPODType<FVector2d> { enum { Value = true }; };
template <> struct TIsUECoreVariant<FVector2d> { enum { Value = true }; };
template<> struct TCanBulkSerialize<FVector2d> { enum { Value = true }; };


#ifdef _MSC_VER
#pragma warning (pop)
#endif
