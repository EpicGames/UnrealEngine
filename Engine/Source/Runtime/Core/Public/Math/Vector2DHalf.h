// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Math/Vector2D.h"
#include "Math/Float16.h"

/**
 * Structure for two dimensional vectors with half floating point precision.
 */
struct FVector2DHalf 
{
	/** Holds the vector's X-component. */
	FFloat16 X;

	/** Holds the vector's Y-component. */
	FFloat16 Y;

public:

	/** Default Constructor (no initialization). */
	[[nodiscard]] UE_FORCEINLINE_HINT FVector2DHalf() { }

	/**
	 * Constructor.
	 *
	 * InX half float X value
	 * Iny half float Y value
	 */
 	[[nodiscard]] UE_FORCEINLINE_HINT FVector2DHalf( const FFloat16& InX,const FFloat16& InY );

	/** Constructor 
	 *
	 * InX float X value
	 * Iny float Y value
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT FVector2DHalf( float InX,float InY );

	/** Constructor 
	 *
	 * Vector2D float vector
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT FVector2DHalf( const FVector2f& Vector2D );
	[[nodiscard]] UE_FORCEINLINE_HINT FVector2DHalf( const FVector2d& Vector2D );// LWC_TODO: This should probably be explicit

public:

	/**
	 * Assignment operator.
	 *
	 * @param Vector2D The value to assign.
	 */
 	FVector2DHalf& operator=( const FVector2f& Vector2D );
	FVector2DHalf& operator=( const FVector2d& Vector2D );	// LWC_TODO: This should probably be explicit

	/** Implicit conversion operator for conversion to FVector2D. */
	operator FVector2f() const;
	operator FVector2d() const;

	/** Conversion with backwards-compatible Truncate rounding mode (default is RTNE) */
	void SetTruncate( float InX, float InY );
	void SetTruncate( const FVector2f& Vector2D );
	void SetTruncate( const FVector2d& Vector2D );

	[[nodiscard]] inline uint32 AsUInt32() const
	{
		uint32 PackedData = (X.Encoded & 0xFFFFFFFF) | (Y.Encoded << 16);
		return PackedData;
	}

public:

	/**
	 * Get a textual representation of the vector.
	 *
	 * @return Text describing the vector.
	 */
	[[nodiscard]] FString ToString() const;

public:

	/**
	 * Serializes the FVector2DHalf.
	 *
	 * @param Ar Reference to the serialization archive.
	 * @param V Reference to the FVector2DHalf being serialized.
	 * @return Reference to the Archive after serialization.
	 */
	friend FArchive& operator<<( FArchive& Ar, FVector2DHalf& V )
	{
		return Ar << V.X << V.Y;
	}
};


/* FVector2DHalf inline functions
 *****************************************************************************/

UE_FORCEINLINE_HINT FVector2DHalf::FVector2DHalf( const FFloat16& InX, const FFloat16& InY )
 	:	X(InX), Y(InY)
{ }


UE_FORCEINLINE_HINT FVector2DHalf::FVector2DHalf( float InX, float InY )
	:	X(InX), Y(InY)
{ }

UE_FORCEINLINE_HINT FVector2DHalf::FVector2DHalf( const FVector2f& Vector2D )
	:	X(Vector2D.X), Y(Vector2D.Y)
{ }

// LWC_TODO: Precision loss.
UE_FORCEINLINE_HINT FVector2DHalf::FVector2DHalf( const FVector2d& Vector2D )
	:	X((float)Vector2D.X), Y((float)Vector2D.Y)
{ }

inline FVector2DHalf& FVector2DHalf::operator=( const FVector2f& Vector2D )
{
 	X = FFloat16(Vector2D.X);
 	Y = FFloat16(Vector2D.Y);

	return *this;
}

// LWC_TODO: Precision loss.
inline FVector2DHalf& FVector2DHalf::operator=( const FVector2d& Vector2D )
{
	X = FFloat16((float)Vector2D.X);
	Y = FFloat16((float)Vector2D.Y);

	return *this;
}

UE_FORCEINLINE_HINT FString FVector2DHalf::ToString() const
{
	return FString::Printf(TEXT("X=%3.3f Y=%3.3f"), (float)X, (float)Y );
}


UE_FORCEINLINE_HINT FVector2DHalf::operator FVector2f() const
{
	return FVector2f((float)X,(float)Y);
}

UE_FORCEINLINE_HINT FVector2DHalf::operator FVector2d() const
{
	return FVector2d((float)X,(float)Y);
}

inline void FVector2DHalf::SetTruncate( float InX, float InY )
{
	X.SetTruncate(InX);
	Y.SetTruncate(InY);
}

UE_FORCEINLINE_HINT void FVector2DHalf::SetTruncate( const FVector2f& Vector2D )
{
	SetTruncate(Vector2D.X,Vector2D.Y);
}

UE_FORCEINLINE_HINT void FVector2DHalf::SetTruncate( const FVector2d& Vector2D )
{
	SetTruncate((float)Vector2D.X,(float)Vector2D.Y);
}