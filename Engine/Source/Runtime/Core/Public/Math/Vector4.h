// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Crc.h"
#include "Math/MathFwd.h" // IWYU pragma: export
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Misc/Parse.h"
#include "Misc/LargeWorldCoordinatesSerializer.h"
#include "Logging/LogMacros.h"
#include "Math/Vector2D.h"
#include "Math/Vector.h"
#include "Serialization/MemoryLayout.h"
#include "Templates/Requires.h"

#include <type_traits>

namespace UE
{
namespace Math
{

/**
 * A 4D homogeneous vector, 4x1 FLOATs, 16-byte aligned.
 */
template<typename T>
struct alignas(16) TVector4
{
	// Can't have a UE_REQUIRES in the declaration because of the forward declarations, so check for allowed types here.
	static_assert(std::is_floating_point_v<T>, "TVector4 only supports float and double types.");


public:
	using FReal = T;

	union
	{
		struct
		{
			/** The vector's X-component. */
			T X;

			/** The vector's Y-component. */
			T Y;

			/** The vector's Z-component. */
			T Z;

			/** The vector's W-component. */
			T W;
		};

		UE_DEPRECATED(all, "For internal use only")
		T XYZW[4];
	};

	/** The number of components this vector type has. */
	CORE_API static constexpr int32 NumComponents = 4;

public:

	/**
	 * Constructor from 3D TVector. W is set to 1.
	 *
	 * @param InVector 3D Vector to set first three components.
	 */
	[[nodiscard]] TVector4(const UE::Math::TVector<T>& InVector);

	/**
	 * Constructor.
	 *
	 * @param InVector 3D Vector to set first three components.
	 * @param InW W Coordinate.
	 */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	[[nodiscard]] inline TVector4(const UE::Math::TVector<T>& InVector, FArg InW)
		: X(InVector.X)
		, Y(InVector.Y)
		, Z(InVector.Z)
		, W((T)InW)
	{
		DiagnosticCheckNaN();
	}

	/**
	 * Constructor allowing copying of an TVector4 whilst setting up a new W component.
	 * @param InVector 4D Vector to set first three components.
	 * @param InOverrideW Replaces W Coordinate of InVector.
	 */

	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	[[nodiscard]] inline TVector4(const UE::Math::TVector4<T>& InVector, FArg OverrideW)
		: X(InVector.X)
		, Y(InVector.Y)
		, Z(InVector.Z)
		, W((T)OverrideW)
	{
		DiagnosticCheckNaN();
	}

	/**
	 * Creates and initializes a new vector from a color value.
	 *
	 * @param InColour Color used to set vector.
	 */
	[[nodiscard]] inline TVector4(const FLinearColor& InColor)
		: X(InColor.R)
		, Y(InColor.G)
		, Z(InColor.B)
		, W(InColor.A)
	{
		DiagnosticCheckNaN();
	}

	/**
	 * Creates and initializes a new vector from a color RGB and W
	 *
	 * @param InColour Color used to set XYZ.
	 * @param InOverrideW
	 */
	[[nodiscard]] inline TVector4(const FLinearColor& InColor, T InOverrideW)
		: X(InColor.R)
		, Y(InColor.G)
		, Z(InColor.B)
		, W(InOverrideW)
	{
		DiagnosticCheckNaN();
	}

	/**
	 * Creates and initializes a new vector from the specified components.
	 *
	 * @param InX X Coordinate.
	 * @param InY Y Coordinate.
	 * @param InZ Z Coordinate.
	 * @param InW W Coordinate.
	 *
	 * NOTE: This default constructor is unlike TVector, TMatrix etc. in that it
	 *       actually initializes the instance.  Ideally it should be = default;
	 *       in the same way, but this would break backwards compatibility.
	 */
	[[nodiscard]] explicit TVector4(T InX = 0.0f, T InY = 0.0f, T InZ = 0.0f, T InW = 1.0f);

	/**
	 * Creates and initializes a new vector from the specified 2D vectors.
	 *
	 * @param InXY A 2D vector holding the X- and Y-components.
	 * @param InZW A 2D vector holding the Z- and W-components.
	 */
	[[nodiscard]] explicit TVector4(TVector2<T> InXY, TVector2<T> InZW);

	/**
	 * Creates and initializes a new vector from an int vector value.
	 *
	 * @param InVector IntVector used to set vector.
	 */
	template<typename IntType>
	[[nodiscard]] TVector4(const TIntVector4<IntType>& InVector);

	/**
	 * Creates and initializes a new vector to zero.
	 *
	 * @param EForceInit Force Init Enum.
	 */
	[[nodiscard]] explicit TVector4(EForceInit);

	/**
	 * Creates an uninitialized new vector.
	 *
	 * @param ENoInit Force uninitialized enum.
	 */
	[[nodiscard]] TVector4(ENoInit);

public:

	// To satisfy UE::Geometry type
	[[nodiscard]] static TVector4<T> Zero()
	{
		return TVector4(T(0), T(0), T(0), T(0));
	}

	// To satisfy UE::Geometry type
	[[nodiscard]] static TVector4<T> One()
	{
		return TVector4(T(1), T(1), T(1), T(1));
	}

	/**
	 * Access a specific component of the vector.
	 *
	 * @param ComponentIndex The index of the component.
	 * @return Reference to the desired component.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T& operator[](int32 ComponentIndex);

	/**
	 * Access a specific component of the vector.
	 *
	 * @param ComponentIndex The index of the component.
	 * @return Copy of the desired component.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T operator[](int32 ComponentIndex) const;

	// Unary operators.

	/**
	 * Gets a negated copy of the vector.
	 *
	 * @return A negated copy of the vector.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TVector4<T> operator-() const;

	/**
	 * Gets the result of adding a vector to this.
	 *
	 * @param V The vector to add.
	 * @return The result of vector addition.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TVector4<T> operator+(const TVector4<T>& V) const;

	/**
	 * Adds another vector to this one.
	 *
	 * @param V The other vector to add.
	 * @return Copy of the vector after addition.
	 */
	inline TVector4<T> operator+=(const TVector4<T>& V);

	/**
	 * Gets the result of subtracting a vector from this.
	 *
	 * @param V The vector to subtract.
	 * @return The result of vector subtraction.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TVector4<T> operator-(const TVector4<T>& V) const;

	/**
	 * Subtracts another vector to this one.
	 *
	 * @param V The other vector to subtract.
	 * @return Copy of the vector after subtraction.
	 */
	inline TVector4<T> operator-=(const TVector4<T>& V);

	/**
     * Gets the result of adding to each component of the vector.
     *
     * @param Bias How much to add to each component.
     * @return The result of addition.
     */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	[[nodiscard]] UE_FORCEINLINE_HINT TVector4<T> operator+(FArg Bias) const
	{
		return TVector4<T>(X + (T)Bias, Y + (T)Bias, Z + (T)Bias, W + (T)Bias);
	}

	/**
     * Gets the result of subtracting from each component of the vector.
     *
     * @param Bias How much to subtract from each component.
     * @return The result of subtraction.
     */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	[[nodiscard]] UE_FORCEINLINE_HINT TVector4<T> operator-(FArg Bias) const
	{
		return TVector4<T>(X - (T)Bias, Y - (T)Bias, Z - (T)Bias, W - (T)Bias);
	}

	/**
	 * Gets the result of scaling this vector.
	 *
	 * @param Scale The scaling factor.
	 * @return The result of vector scaling.
	 */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	[[nodiscard]] UE_FORCEINLINE_HINT TVector4<T> operator*(FArg Scale) const
	{
		return TVector4(X * Scale, Y * Scale, Z * Scale, W * Scale);
	}

	/**
	 * Gets the result of dividing this vector.
	 *
	 * @param Scale What to divide by.
	 * @return The result of division.
	 */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	[[nodiscard]] inline TVector4<T> operator/(FArg Scale) const
	{
		const T RScale = T(1.0f) / Scale;
		return TVector4(X * RScale, Y * RScale, Z * RScale, W * RScale);
	}

	/**
	 * Gets the result of dividing this vector.
	 *
	 * @param V What to divide by.
	 * @return The result of division.
	 */
	[[nodiscard]] TVector4<T> operator/(const TVector4<T>& V) const;

	/**
	 * Gets the result of multiplying a vector with this.
	 *
	 * @param V The vector to multiply.
	 * @return The result of vector multiplication.
	 */
	[[nodiscard]] TVector4<T> operator*(const TVector4<T>& V) const;

	/**
	 * Gets the result of multiplying a vector with another Vector (component wise).
	 *
	 * @param V The vector to multiply.
	 * @return The result of vector multiplication.
	 */
	TVector4<T> operator*=(const TVector4<T>& V);

	/**
	 * Gets the result of dividing a vector with another Vector (component wise).
	 *
	 * @param V The vector to divide with.
	 * @return The result of vector multiplication.
	 */
	TVector4<T> operator/=(const TVector4<T>& V);

	/**
	 * Gets the result of scaling this vector.
	 *
	 * @param Scale The scaling factor.
	 * @return The result of vector scaling.
	 */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	inline TVector4<T> operator*=(FArg Scale)
	{
		X *= Scale; Y *= Scale; Z *= Scale; W *= Scale;
		DiagnosticCheckNaN();
		return *this;
	}

	/**
	 * Gets the result of scaling this vector by 1/Scale.
	 *
	 * @param Scale The inverse scaling factor.
	 * @return The result of vector scaling by 1/Scale.
	 */
	template<typename FArg UE_REQUIRES(std::is_arithmetic_v<FArg>)>
	inline TVector4<T> operator/=(FArg Scale)
	{
		const T RV = T(1.0f) / Scale;
		X *= RV; Y *= RV; Z *= RV; W *= RV;
		DiagnosticCheckNaN();
		return *this;
	}

	/**
	 * Checks for equality against another vector.
	 *
	 * @param V The other vector.
	 * @return true if the two vectors are the same, otherwise false.
	 */
	[[nodiscard]] bool operator==(const TVector4<T>& V) const;

	/**
	 * Checks for inequality against another vector.
	 *
	 * @param V The other vector.
	 * @return true if the two vectors are different, otherwise false.
	 */
	[[nodiscard]] bool operator!=(const TVector4<T>& V) const;

	/**
	 * Calculate Cross product between this and another vector.
	 *
	 * @param V The other vector.
	 * @return The Cross product.
	 */
	[[nodiscard]] TVector4<T> operator^(const TVector4<T>& V) const;

public:

	// Simple functions.

	/**
	 * Gets a specific component of the vector.
	 *
	 * @param Index The index of the component.
	 * @return Reference to the component.
	 */
	[[nodiscard]] T& Component(int32 Index);

	/**
	* Gets a specific component of the vector.
	*
	* @param Index The index of the component.
	* @return Reference to the component.
	*/
	[[nodiscard]] const T& Component(int32 Index) const;

	/**
	 * Tests if index is valid, i.e. greater than or equal to zero, and less than the number of components in the vector.
	 *
	 * @param Index Index to test.
	 * @returns True if index is valid. False otherwise.
	 */
	[[nodiscard]] bool IsValidIndex(int32 Index) const;

	/**
	 * Error tolerant comparison.
	 *
	 * @param V Vector to compare against.
	 * @param Tolerance Error Tolerance.
	 * @return true if the two vectors are equal within specified tolerance, otherwise false.
	 */
	[[nodiscard]] bool Equals(const TVector4<T>& V, T Tolerance=UE_KINDA_SMALL_NUMBER) const;

	/**
	 * Check if the vector is of unit length, with specified tolerance.
	 *
	 * @param LengthSquaredTolerance Tolerance against squared length.
	 * @return true if the vector is a unit vector within the specified tolerance.
	 */
	[[nodiscard]] bool IsUnit3(T LengthSquaredTolerance = UE_KINDA_SMALL_NUMBER) const;

	/**
	 * Get a textual representation of the vector.
	 *
	 * @return Text describing the vector.
	 */
	[[nodiscard]] FString ToString() const;

	/**
	 * Initialize this Vector based on an FString. The String is expected to contain X=, Y=, Z=, W=.
	 * The TVector4 will be bogus when InitFromString returns false.
	 *
	 * @param InSourceString	FString containing the vector values.
	 * @return true if the X,Y,Z values were read successfully; false otherwise.
	 */
	bool InitFromString(const FString& InSourceString);

	/**
	 * Returns a normalized copy of the vector if safe to normalize.
	 *
	 * @param Tolerance Minimum squared length of vector for normalization.
	 * @return A normalized copy of the vector or a zero vector.
	 */
	[[nodiscard]] inline TVector4 GetSafeNormal(T Tolerance = UE_SMALL_NUMBER) const;

	/**
	 * Calculates normalized version of vector without checking if it is non-zero.
	 *
	 * @return Normalized version of vector.
	 */
	[[nodiscard]] inline TVector4 GetUnsafeNormal3() const;

	/**
	 * Return the FRotator orientation corresponding to the direction in which the vector points.
	 * Sets Yaw and Pitch to the proper numbers, and sets roll to zero because the roll can't be determined from a vector.
	 * @return FRotator from the Vector's direction.
	 */
	[[nodiscard]] CORE_API TRotator<T> ToOrientationRotator() const;

	/**
	 * Return the Quaternion orientation corresponding to the direction in which the vector points.
	 * @return Quaternion from the Vector's direction.
	 */
	[[nodiscard]] CORE_API TQuat<T> ToOrientationQuat() const;

	/**
	 * Return the FRotator orientation corresponding to the direction in which the vector points.
	 * Sets Yaw and Pitch to the proper numbers, and sets roll to zero because the roll can't be determined from a vector.
	 * Identical to 'ToOrientationRotator()'.
	 * @return FRotator from the Vector's direction.
	 * @see ToOrientationRotator()
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TRotator<T> Rotation() const
	{
		return ToOrientationRotator();
	}

	/**
	 * Set all of the vectors coordinates.
	 *
	 * @param InX New X Coordinate.
	 * @param InY New Y Coordinate.
	 * @param InZ New Z Coordinate.
	 * @param InW New W Coordinate.
	 */
	inline void Set(T InX, T InY, T InZ, T InW);

	/** Gets the component-wise min of two vectors. */
	[[nodiscard]] TVector4<T> ComponentMin(const TVector4<T>& Other) const;

	/** Gets the component-wise max of two vectors. */
	[[nodiscard]] TVector4<T> ComponentMax(const TVector4<T>& Other) const;

	/**
	 * Get the length of this vector not taking W component into account.
	 *
	 * @return The length of this vector.
	 */
	[[nodiscard]] T Size3() const;

	/**
	 * Get the squared length of this vector not taking W component into account.
	 *
	 * @return The squared length of this vector.
	 */
	[[nodiscard]] T SizeSquared3() const;

	/**
	 * Get the length (magnitude) of this vector, taking the W component into account
	 *
	 * @return The length of this vector
	 */
	[[nodiscard]] T Size() const;

	/**
	 * Get the squared length of this vector, taking the W component into account
	 *
	 * @return The squared length of this vector
	 */
	[[nodiscard]] T SizeSquared() const;

	/** Utility to check if there are any non-finite values (NaN or Inf) in this vector. */
	[[nodiscard]] bool ContainsNaN() const;

	/** Utility to check if the XYZ components of this vector are nearly zero given the tolerance. */
	[[nodiscard]] bool IsNearlyZero3(T Tolerance = UE_KINDA_SMALL_NUMBER) const;

	/** Utility to check if all of the components of this vector are nearly zero given the tolerance. */
	[[nodiscard]] bool IsNearlyZero(T Tolerance = UE_KINDA_SMALL_NUMBER) const;

	/** Reflect vector. */
	[[nodiscard]] TVector4<T> Reflect3(const TVector4<T>& Normal) const;

	/**
	 * Find good arbitrary axis vectors to represent U and V axes of a plane,
	 * given just the normal.
	 */
	void FindBestAxisVectors3(TVector4<T>& Axis1, TVector4<T>& Axis2) const;

#if ENABLE_NAN_DIAGNOSTIC
	inline void DiagnosticCheckNaN()
	{
		if (ContainsNaN())
		{
			logOrEnsureNanError(TEXT("FVector4 contains NaN: %s"), *ToString());
		}
	}
#else
	UE_FORCEINLINE_HINT void DiagnosticCheckNaN() { }
#endif

private:
	bool SerializeFromVector3(FName StructTag, FArchive&Ar);
public:

	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	bool SerializeFromMismatchedTag(FName StructTag, FArchive& Ar)
	{
		if(SerializeFromVector3(StructTag, Ar))
		{
			return true;
		}

		if constexpr (std::is_same_v<T, float>)
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Vector4, Vector4f, Vector4d);
		}
		else
		{
			return UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(Ar, Vector4, Vector4d, Vector4f);
		}
	}
	
	// Conversion from other type: double->float
	template<typename FArg UE_REQUIRES(std::is_same_v<FArg, double> && std::is_same_v<T, float>)>
	explicit TVector4(const TVector4<FArg>& From) : TVector4<T>((T)From.X, (T)From.Y, (T)From.Z, (T)From.W) {}

	// Conversion from other type: float->double
	template<typename FArg UE_REQUIRES(std::is_same_v<FArg, float> && std::is_same_v<T, double>)>
	explicit TVector4(const TVector4<FArg>& From) : TVector4<T>((T)From.X, (T)From.Y, (T)From.Z, (T)From.W) {}

	/**
	 * Creates a hash value from a TVector4.
	 *
	 * @param Vector the vector to create a hash value for
	 *
	 * @return The hash value from the components
	 */
	UE_FORCEINLINE_HINT friend uint32 GetTypeHash(const UE::Math::TVector4<T>& Vector)
	{
		return FCrc::MemCrc_DEPRECATED(&Vector, sizeof(Vector));
	}
};

/**
 * Serializer.
 *
 * @param Ar The Serialization Archive.
 * @param V The vector being serialized.
 * @return Reference to the Archive after serialization.
 */
inline FArchive& operator<<(FArchive& Ar, TVector4<float>& V)
{
	return Ar << V.X << V.Y << V.Z << V.W;
}

inline FArchive& operator<<(FArchive& Ar, TVector4<double>& V)
{
	if (Ar.UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
	{
		Ar << V.X << V.Y << V.Z << V.W;
	}
	else
	{
		checkf(Ar.IsLoading(), TEXT("float -> double conversion applied outside of load!"));
		// Stored as floats, so serialize float and copy.
		float X, Y, Z, W;
		Ar << X << Y << Z << W;
		V = TVector4<double>(X, Y, Z, W);
	}
	return Ar;
}


/* TVector4 inline functions
 *****************************************************************************/

template<typename T>
inline TVector4<T>::TVector4(const UE::Math::TVector<T>& InVector)
	: X(InVector.X)
	, Y(InVector.Y)
	, Z(InVector.Z)
	, W(1.0f)
{
	DiagnosticCheckNaN();
}

template<typename T>
inline TVector4<T>::TVector4(T InX, T InY, T InZ, T InW)
	: X(InX)
	, Y(InY)
	, Z(InZ)
	, W(InW)
{
	DiagnosticCheckNaN();
}


template<typename T>
inline TVector4<T>::TVector4(EForceInit)
	: X(0.f)
	, Y(0.f)
	, Z(0.f)
	, W(0.f)
{
	DiagnosticCheckNaN();
}


template<typename T>
UE_FORCEINLINE_HINT TVector4<T>::TVector4(ENoInit)
{
}


template<typename T>
inline TVector4<T>::TVector4(TVector2<T> InXY, TVector2<T> InZW)
	: X(InXY.X)
	, Y(InXY.Y)
	, Z(InZW.X)
	, W(InZW.Y)
{
	DiagnosticCheckNaN();
}

template<typename T>
template<typename IntType>
UE_FORCEINLINE_HINT TVector4<T>::TVector4(const TIntVector4<IntType>& InVector)
	: X((T)InVector.X)
	, Y((T)InVector.Y)
	, Z((T)InVector.Z)
	, W((T)InVector.W)
{
}

template<typename T>
UE_FORCEINLINE_HINT T& TVector4<T>::operator[](int32 ComponentIndex)
{
	return Component(ComponentIndex);
}


template<typename T>
UE_FORCEINLINE_HINT T TVector4<T>::operator[](int32 ComponentIndex) const
{
	return Component(ComponentIndex);
}


template<typename T>
inline void TVector4<T>::Set(T InX, T InY, T InZ, T InW)
{
	X = InX;
	Y = InY;
	Z = InZ;
	W = InW;

	DiagnosticCheckNaN();
}


template<typename T>
UE_FORCEINLINE_HINT TVector4<T> TVector4<T>::operator-() const
{
	return TVector4(-X, -Y, -Z, -W);
}


template<typename T>
UE_FORCEINLINE_HINT TVector4<T> TVector4<T>::operator+(const TVector4<T>& V) const
{
	return TVector4(X + V.X, Y + V.Y, Z + V.Z, W + V.W);
}


template<typename T>
inline TVector4<T> TVector4<T>::operator+=(const TVector4<T>& V)
{
	X += V.X; Y += V.Y; Z += V.Z; W += V.W;
	DiagnosticCheckNaN();
	return *this;
}


template<typename T>
UE_FORCEINLINE_HINT TVector4<T> TVector4<T>::operator-(const TVector4<T>& V) const
{
	return TVector4(X - V.X, Y - V.Y, Z - V.Z, W - V.W);
}


template<typename T>
inline TVector4<T> TVector4<T>::operator-=(const TVector4<T>& V)
{
	X -= V.X; Y -= V.Y; Z -= V.Z; W -= V.W;
	DiagnosticCheckNaN();
	return *this;
}


template<typename T>
UE_FORCEINLINE_HINT TVector4<T> TVector4<T>::operator*(const TVector4<T>& V) const
{
	return TVector4(X * V.X, Y * V.Y, Z * V.Z, W * V.W);
}


template<typename T>
inline TVector4<T> TVector4<T>::operator^(const TVector4<T>& V) const
{
	return TVector4(
		Y * V.Z - Z * V.Y,
		Z * V.X - X * V.Z,
		X * V.Y - Y * V.X,
		T(0.0f)
	);
}


template<typename T>
inline T& TVector4<T>::Component(int32 Index)
{
	check(IsValidIndex(Index));
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return XYZW[Index];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

template<typename T>
inline const T& TVector4<T>::Component(int32 Index) const
{
	check(IsValidIndex(Index));
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return XYZW[Index];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

template<typename T>
bool TVector4<T>::IsValidIndex(int32 Index) const
{
	return (Index >= 0) && (Index < NumComponents);
}

template<typename T>
UE_FORCEINLINE_HINT bool TVector4<T>::operator==(const TVector4<T>& V) const
{
	return ((X == V.X) && (Y == V.Y) && (Z == V.Z) && (W == V.W));
}


template<typename T>
UE_FORCEINLINE_HINT bool TVector4<T>::operator!=(const TVector4<T>& V) const
{
	return ((X != V.X) || (Y != V.Y) || (Z != V.Z) || (W != V.W));
}


template<typename T>
UE_FORCEINLINE_HINT bool TVector4<T>::Equals(const TVector4<T>& V, T Tolerance) const
{
	return FMath::Abs(X-V.X) <= Tolerance && FMath::Abs(Y-V.Y) <= Tolerance && FMath::Abs(Z-V.Z) <= Tolerance && FMath::Abs(W-V.W) <= Tolerance;
}


template<typename T>
UE_FORCEINLINE_HINT FString TVector4<T>::ToString() const
{
	return FString::Printf(TEXT("X=%3.3f Y=%3.3f Z=%3.3f W=%3.3f"), X, Y, Z, W);
}


template<typename T>
inline bool TVector4<T>::InitFromString(const FString& InSourceString)
{
	X = Y = Z = 0;
	W = 1.0f;

	// The initialization is only successful if the X, Y, and Z values can all be parsed from the string
	const bool bSuccessful = FParse::Value(*InSourceString, TEXT("X=") , X) && FParse::Value(*InSourceString, TEXT("Y="), Y) && FParse::Value(*InSourceString, TEXT("Z="), Z);

	// W is optional, so don't factor in its presence (or lack thereof) in determining initialization success
	FParse::Value(*InSourceString, TEXT("W="), W);

	return bSuccessful;
}


template<typename T>
inline TVector4<T> TVector4<T>::GetSafeNormal(T Tolerance) const
{
	const T SquareSum = X*X + Y*Y + Z*Z;
	if(SquareSum > Tolerance)
	{
		const T Scale = FMath::InvSqrt(SquareSum);
		return TVector4(X*Scale, Y*Scale, Z*Scale, 0.0f);
	}
	return TVector4(T(0.f));
}


template<typename T>
inline TVector4<T> TVector4<T>::GetUnsafeNormal3() const
{
	const T Scale = FMath::InvSqrt(X*X+Y*Y+Z*Z);
	return TVector4(X*Scale, Y*Scale, Z*Scale, 0.0f);
}

template<typename T>
UE_FORCEINLINE_HINT TVector4<T> TVector4<T>::ComponentMin(const TVector4<T>& Other) const
{
    return TVector4<T>(FMath::Min(X, Other.X), FMath::Min(Y, Other.Y), FMath::Min(Z, Other.Z), FMath::Min(W, Other.W));
}

template<typename T>
UE_FORCEINLINE_HINT TVector4<T> TVector4<T>::ComponentMax(const TVector4<T>& Other) const
{
    return TVector4<T>(FMath::Max(X, Other.X), FMath::Max(Y, Other.Y), FMath::Max(Z, Other.Z), FMath::Max(W, Other.W));
}

template<typename T>
UE_FORCEINLINE_HINT T TVector4<T>::Size3() const
{
	return FMath::Sqrt(X*X + Y*Y + Z*Z);
}


template<typename T>
UE_FORCEINLINE_HINT T TVector4<T>::SizeSquared3() const
{
	return X*X + Y*Y + Z*Z;
}

template<typename T>
UE_FORCEINLINE_HINT T TVector4<T>::Size() const
{
	return FMath::Sqrt(X*X + Y*Y + Z*Z + W*W);
}

template<typename T>
UE_FORCEINLINE_HINT T TVector4<T>::SizeSquared() const
{
	return X*X + Y*Y + Z*Z + W*W;
}


template<typename T>
UE_FORCEINLINE_HINT bool TVector4<T>::IsUnit3(T LengthSquaredTolerance) const
{
	return FMath::Abs(1.0f - SizeSquared3()) < LengthSquaredTolerance;
}


template<typename T>
inline bool TVector4<T>::ContainsNaN() const
{
	return (!FMath::IsFinite(X) || 
			!FMath::IsFinite(Y) ||
			!FMath::IsFinite(Z) ||
			!FMath::IsFinite(W));
}


template<typename T>
inline bool TVector4<T>::IsNearlyZero3(T Tolerance) const
{
	return
			FMath::Abs(X)<=Tolerance
		&&	FMath::Abs(Y)<=Tolerance
		&&	FMath::Abs(Z)<=Tolerance;
}

template<typename T>
inline bool TVector4<T>::IsNearlyZero(T Tolerance) const
{
	return
			FMath::Abs(X)<=Tolerance
		&&	FMath::Abs(Y)<=Tolerance
		&&	FMath::Abs(Z)<=Tolerance
		&&	FMath::Abs(W)<=Tolerance;
}

template<typename T>
inline TVector4<T> TVector4<T>::operator*=(const TVector4<T>& V)
{
	X *= V.X; Y *= V.Y; Z *= V.Z; W *= V.W;
	DiagnosticCheckNaN();
	return *this;
}


template<typename T>
inline TVector4<T> TVector4<T>::operator/=(const TVector4<T>& V)
{
	X /= V.X; Y /= V.Y; Z /= V.Z; W /= V.W;
	DiagnosticCheckNaN();
	return *this;
}


template<typename T>
UE_FORCEINLINE_HINT TVector4<T> TVector4<T>::operator/(const TVector4<T>& V) const
{
	return TVector4(X / V.X, Y / V.Y, Z / V.Z, W / V.W);
}


template <typename T>
bool TVector4<T>::SerializeFromVector3(FName StructTag, FArchive& Ar)
{
	// Upgrade Vector3 - only set X/Y/Z.  The W should already have been set to the property specific default and we don't want to trash it by forcing 0 or 1.
	if(StructTag == NAME_Vector3f)
	{
		FVector3f AsVec;
		Ar << AsVec;
		X = (T)AsVec.X;
		Y = (T)AsVec.Y;
		Z = (T)AsVec.Z;		
		return true;
	}
	else if(StructTag == NAME_Vector || StructTag == NAME_Vector3d)
	{
		FVector3d AsVec;	// Note: Vector relies on FVector3d serializer to handle float/double based on archive version.
		Ar << AsVec;
		X = (T)AsVec.X;
		Y = (T)AsVec.Y;
		Z = (T)AsVec.Z;		
		return true;
	}
	return false;
}


} // namespace UE::Math
} // namespace UE


UE_DECLARE_LWC_TYPE(Vector4);

template<> struct TIsPODType<FVector4f> { enum { Value = true }; };
template<> struct TIsPODType<FVector4d> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FVector4f> { enum { Value = true }; };
template<> struct TIsUECoreVariant<FVector4d> { enum { Value = true }; };
template<> struct TCanBulkSerialize<FVector4f> { enum { Value = true }; };
template<> struct TCanBulkSerialize<FVector4d> { enum { Value = true }; };
DECLARE_INTRINSIC_TYPE_LAYOUT(FVector4f);
DECLARE_INTRINSIC_TYPE_LAYOUT(FVector4d);

/**
 * Calculates 3D Dot product of two 4D vectors.
 *
 * @param V1 The first vector.
 * @param V2 The second vector.
 * @return The 3D Dot product.
 */

template<typename T>
UE_FORCEINLINE_HINT T Dot3(const UE::Math::TVector4<T>& V1, const UE::Math::TVector4<T>& V2)
{
	return V1.X * V2.X + V1.Y * V2.Y + V1.Z * V2.Z;
}


/**
 * Calculates 3D Dot product of one 4D vectors and one 3D vector
 *
 * @param V1 The first vector.
 * @param V2 The second vector.
 * @return The 3D Dot product.
 */

template<typename T>
UE_FORCEINLINE_HINT T Dot3(const UE::Math::TVector4<T>& V1, const UE::Math::TVector<T>& V2)
{
	return V1.X * V2.X + V1.Y * V2.Y + V1.Z * V2.Z;
}

template<typename T>
UE_FORCEINLINE_HINT T Dot3(const UE::Math::TVector<T>& V1, const UE::Math::TVector4<T>& V2)
{
	return V1.X * V2.X + V1.Y * V2.Y + V1.Z * V2.Z;
}

/**
 * Calculates 4D Dot product.
 *
 * @param V1 The first vector.
 * @param V2 The second vector.
 * @return The 4D Dot Product.
 */
template<typename T>
UE_FORCEINLINE_HINT T Dot4(const UE::Math::TVector4<T>& V1, const UE::Math::TVector4<T>& V2)
{
	return V1.X * V2.X + V1.Y * V2.Y + V1.Z * V2.Z + V1.W * V2.W;
}

namespace UE
{
namespace Math
{
	/**
	 * Scales a vector.
	 *
	 * @param Scale The scaling factor.
	 * @param V The vector to scale.
	 * @return The result of scaling.
	 */
	template<typename T, typename T2 UE_REQUIRES(std::is_arithmetic_v<T2>)>
	UE_FORCEINLINE_HINT TVector4<T> operator*(T2 Scale, const TVector4<T>& V)
	{
		return V.operator*(Scale);
	}


	template<typename T>
	UE_FORCEINLINE_HINT TVector4<T> TVector4<T>::Reflect3(const TVector4<T>& Normal) const
	{
		return T(2.0f) * Dot3(*this, Normal) * Normal - *this;
	}


	template<typename T>
	inline void TVector4<T>::FindBestAxisVectors3(TVector4<T>& Axis1, TVector4<T>& Axis2) const
	{
		const T NX = FMath::Abs(X);
		const T NY = FMath::Abs(Y);
		const T NZ = FMath::Abs(Z);

		// Find best basis vectors.
		if (NZ > NX && NZ > NY)	Axis1 = TVector4(1, 0, 0);
		else					Axis1 = TVector4(0, 0, 1);

		Axis1 = (Axis1 - *this * Dot3(Axis1, *this)).GetSafeNormal();
		Axis2 = Axis1 ^ *this;
	}


/* FVector inline functions
*****************************************************************************/

template<typename T>
inline TVector<T>::TVector( const TVector4<T>& V )
	: X(V.X), Y(V.Y), Z(V.Z)
	{
		DiagnosticCheckNaN();
	}


/* FVector2D inline functions
*****************************************************************************/

template<typename T>
inline TVector2<T>::TVector2( const TVector4<T>& V )
	: X(V.X), Y(V.Y)
	{
		DiagnosticCheckNaN();
	}

} // namespace UE::Math
} // namespace UE


