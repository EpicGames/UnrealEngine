// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"

// If defined, allow double->float conversion in some VectorStore functions.
#define SUPPORT_DOUBLE_TO_FLOAT_VECTOR_CONVERSION 1

// Platform specific vector intrinsics include.
#if PLATFORM_ENABLE_VECTORINTRINSICS_NEON
#include "Math/UnrealMathNeon.h"
#elif defined(__cplusplus_cli)
#include "Math/UnrealMathFPU.h" // there are compile issues with UnrealMathSSE in managed mode, so use the FPU version
#elif PLATFORM_ENABLE_VECTORINTRINSICS
#include "Math/UnrealMathSSE.h"
#else
#include "Math/UnrealMathFPU.h"
#endif

#define SIMD_ALIGNMENT (alignof(VectorRegister))


namespace UE::Math::VectorRegisterPrivate
{
	// Have to do this with partial specializations to work around Clang bug on Windows
	template<typename T>
	struct TVectorRegisterTypeHelper
	{
		using Type = void;
	};

	template<>
	struct TVectorRegisterTypeHelper<float>
	{
		using Type = VectorRegister4Float;
	};

	template<>
	struct TVectorRegisterTypeHelper<double>
	{
		using Type = VectorRegister4Double;
	};
}

// Alias which helps resolve a template type (float/double) to the appropriate VectorRegister type (VectorRegister4Float/VectorRegister4Double).
template<typename T>
using TVectorRegisterType = typename UE::Math::VectorRegisterPrivate::TVectorRegisterTypeHelper<T>::Type;

// 'Cross-platform' vector intrinsics (built on the platform-specific ones defined above)
#include "Math/UnrealMathVectorCommon.h.inl"


#if !defined(UE_SSE_DOUBLE_ALIGNMENT) || (UE_SSE_DOUBLE_ALIGNMENT <= 16)
// Suitable to just use regular VectorRegister4Double as persistent variables.
using PersistentVectorRegister4Double = VectorRegister4Double;
#else
/**
 * VectorRegister type to be used as member variables of structs, when alignment should not be over 16 bytes.
 * 32-byte alignment requires large allocations in some allocators, and this allows us to keep at a 16-byte aligned type in situations (ie AVX) where the underlying
 * VectorRegister type might want 32-byte alignment instead. It does imply use of unaligned loads/stores (which should be fine) in the conversion process, but try to
 * formulate code using these types to only load/store once by avoiding repeated type conversion.
 *
 * See TPersistentVectorRegisterType<> below, for automatic selection of the correct type for a member variable.
 */
struct alignas(16) PersistentVectorRegister4Double
{
	double XYZW[4];

	UE_FORCEINLINE_HINT PersistentVectorRegister4Double() = default;

	UE_FORCEINLINE_HINT PersistentVectorRegister4Double(const VectorRegister4Double& Register)
	{
		VectorStore(Register, XYZW);
	}

	UE_FORCEINLINE_HINT PersistentVectorRegister4Double(const VectorRegister4Float& Register)
	{
		VectorStore(VectorRegister4Double(Register), XYZW);
	}

	inline PersistentVectorRegister4Double& operator=(const VectorRegister4Double& Register)
	{
		VectorStore(Register, XYZW);
		return *this;
	}

	inline PersistentVectorRegister4Double& operator=(const VectorRegister4Float& Register)
	{
		VectorStore(VectorRegister4Double(Register), XYZW);
		return *this;
	}

	UE_FORCEINLINE_HINT operator VectorRegister4Double() const
	{
		return VectorLoad(XYZW);
	}
};
#endif // Alignment check for VectorRegister4Double

// Suitable to just use regular VectorRegister4Float as persistent variables.
using PersistentVectorRegister4Float = VectorRegister4Float;

/**
 * Alias for VectorRegister type to be used as member variables of structs, when alignment should not be over 16 bytes.
 * 32-byte alignment requires large allocations in some allocators, which we want to avoid.
 * See PersistentVectorRegister4Double for more details.
 * 
 * Usage example:
 * TPersistentVectorRegisterType<float> MyFloatRegister;
 * TPersistentVectorRegisterType<double> MyDoubleRegister;
 */
template<typename T>
using TPersistentVectorRegisterType = std::conditional_t<std::is_same_v<T, float>, PersistentVectorRegister4Float, std::conditional_t<std::is_same_v<T, double>, PersistentVectorRegister4Double, void> >;



/** Vector that represents (1/255,1/255,1/255,1/255) */
inline constexpr VectorRegister VECTOR_INV_255 = MakeVectorRegisterDoubleConstant(1.0 / 255, 1.0 / 255, 1.0 / 255, 1.0 / 255);

// Old names for comparison functions, kept for compatibility.
#define VectorMask_LT( Vec1, Vec2 )			VectorCompareLT(Vec1, Vec2)
#define VectorMask_LE( Vec1, Vec2 )			VectorCompareLE(Vec1, Vec2)
#define VectorMask_GT( Vec1, Vec2 )			VectorCompareGT(Vec1, Vec2)
#define VectorMask_GE( Vec1, Vec2 )			VectorCompareGE(Vec1, Vec2)
#define VectorMask_EQ( Vec1, Vec2 )			VectorCompareEQ(Vec1, Vec2)
#define VectorMask_NE( Vec1, Vec2 )			VectorCompareNE(Vec1, Vec2)

/**
* Below this weight threshold, animations won't be blended in.
*/
#define ZERO_ANIMWEIGHT_THRESH (0.00001f)
#define ZERO_ANIMWEIGHT_THRESH_DOUBLE (0.00001)

namespace GlobalVectorConstants
{
	inline constexpr VectorRegister AnimWeightThreshold = MakeVectorRegisterConstant(ZERO_ANIMWEIGHT_THRESH_DOUBLE, ZERO_ANIMWEIGHT_THRESH_DOUBLE, ZERO_ANIMWEIGHT_THRESH_DOUBLE, ZERO_ANIMWEIGHT_THRESH_DOUBLE);
	inline constexpr VectorRegister RotationSignificantThreshold = MakeVectorRegisterConstant(1.0 - UE_DELTA*UE_DELTA, 1.0 - UE_DELTA*UE_DELTA, 1.0 - UE_DELTA*UE_DELTA, 1.0 - UE_DELTA*UE_DELTA);
}

struct FScopedFTZFloatMode
{
	FScopedFTZFloatMode()
	{
		ControlRegisterState = VectorGetControlRegister();
		VectorSetControlRegister(ControlRegisterState | VECTOR_DENORMALS_FLUSH_TO_ZERO);
	}

	~FScopedFTZFloatMode()
	{
		VectorSetControlRegister(ControlRegisterState);
	}

private:
	uint32 ControlRegisterState;
};
