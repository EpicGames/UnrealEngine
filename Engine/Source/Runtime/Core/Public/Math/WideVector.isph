// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __WIDEVECTOR_ISPH__
#define __WIDEVECTOR_ISPH__

#include "Math/Vector.isph"

struct WideFVector4f
{
	float V[programCount];
};

struct WideFVector4
{
	double V[programCount];
};

// Define types
typedef WideFVector4 WideFVector4d;

#if TARGET_WIDTH == 4
static const uniform struct WideFVector4d WFloatZero = { {0.0d, 0.0d, 0.0d, 0.0d} };
static const uniform struct WideFVector4d WFloat0001 = { {0.0d, 0.0d, 0.0d, 1.0d} };
static const uniform struct WideFVector4d WSmallLengthThreshold = { {1.d-8, 1.d-8, 1.d-8, 1.d-8} };
#elif TARGET_WIDTH == 8
static const uniform struct WideFVector4d WFloatZero = { {0.0d, 0.0d, 0.0d, 0.0d, 0.0d, 0.0d, 0.0d, 0.0d} };
static const uniform struct WideFVector4d WFloat0001 = { {0.0d, 0.0d, 0.0d, 1.0d, 0.0d, 0.0d, 0.0d, 1.0d} };
static const uniform struct WideFVector4d WSmallLengthThreshold = { {1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8} };
#elif TARGET_WIDTH == 16
static const uniform struct WideFVector4d WFloatZero = { {0.0d, 0.0d, 0.0d, 0.0d, 0.0d, 0.0d, 0.0d, 0.0d, 0.0d, 0.0d, 0.0d, 0.0d, 0.0d, 0.0d, 0.0d, 0.0d} };
static const uniform struct WideFVector4d WFloat0001 = { {0.0d, 0.0d, 0.0d, 1.0d, 0.0d, 0.0d, 0.0d, 1.0d, 0.0d, 0.0d, 0.0d, 1.0d, 0.0d, 0.0d, 0.0d, 1.0d} };
static const uniform struct WideFVector4d WSmallLengthThreshold = { {1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8, 1.d-8} };
#endif

unmasked inline uniform WideFVector4 operator+(const uniform WideFVector4 &A, const uniform WideFVector4 &B)
{
	uniform WideFVector4 Result;

	Result.V[programIndex] = A.V[programIndex] + B.V[programIndex];

	return Result;
}

unmasked inline uniform WideFVector4 operator-(const uniform WideFVector4 &A, const uniform WideFVector4 &B)
{
	uniform WideFVector4 Result;

	Result.V[programIndex] = A.V[programIndex] - B.V[programIndex];

	return Result;
}

unmasked inline uniform WideFVector4 operator*(const uniform WideFVector4 &A, const uniform WideFVector4 &B)
{
	uniform WideFVector4 Result;

	Result.V[programIndex] = A.V[programIndex] * B.V[programIndex];

	return Result;
}

unmasked inline uniform WideFVector4 operator*(const uniform WideFVector4 &A, const uniform FReal F)
{
	uniform WideFVector4 Result;

	Result.V[programIndex] = A.V[programIndex] * F;

	return Result;
}

unmasked inline uniform WideFVector4 VectorSwizzle(const uniform WideFVector4 &Vec, const uniform int X, const uniform int Y, const uniform int Z, const uniform int W)
{
#if TARGET_WIDTH == 4
	const varying int vPerm = { X, Y, Z, W };
#elif TARGET_WIDTH == 8
	const varying int vPerm = { X, Y, Z, W, X+4, Y+4, Z+4, W+4 };
#elif TARGET_WIDTH == 16
	const varying int vPerm = { X, Y, Z, W, X+4, Y+4, Z+4, W+4, X+8, Y+8, Z+8, W+8, X+12, Y+12, Z+12, W+12 };
#endif

	const FReal V = Vec.V[programIndex];
	const FReal R = shuffle(V, vPerm);
	uniform WideFVector4 Result;
	Result.V[programIndex] = R;

	return Result;
}

template<>
inline uniform WideFVector4 VectorReplicate<uniform WideFVector4>(const uniform WideFVector4 &Vec, const uniform int R)
{
	unmasked
	{
	#if TARGET_WIDTH == 4
		const varying int vPerm = { R, R, R, R };
	#elif TARGET_WIDTH == 8
		const varying int vPerm = { R, R, R, R, R+4, R+4, R+4, R+4 };
	#elif TARGET_WIDTH == 16
		const varying int vPerm = { R, R, R, R, R+4, R+4, R+4, R+4, R+8, R+8, R+8, R+8, R+12, R+12, R+12, R+12 };
	#endif

		const FReal V = Vec.V[programIndex];
		const FReal S = shuffle(V, vPerm);
		uniform WideFVector4 Result;
		Result.V[programIndex] = S;

		return Result;
	}
}

unmasked inline uniform WideFVector4 VectorCompareGE(const uniform WideFVector4 &A, const uniform WideFVector4 &B)
{
	uniform WideFVector4 Result;

	Result.V[programIndex] = select(A.V[programIndex] >= B.V[programIndex], REAL_BITS(NAN), REAL_BITS(0));

	return Result;
}

unmasked inline uniform WideFVector4f VectorReciprocalSqrtAccurate(const uniform WideFVector4f& V)
{
	uniform WideFVector4f Result;

	Result.V[programIndex] = rsqrt(V.V[programIndex]);

	return Result;
}

unmasked inline uniform WideFVector4d VectorReciprocalSqrtAccurate(const uniform WideFVector4d& V)
{
	uniform WideFVector4d Result;

	Result.V[programIndex] = 1.0d / sqrt(V.V[programIndex]);

	return Result;
}

unmasked inline uniform WideFVector4 VectorDot4( const uniform WideFVector4& Vec1, const uniform WideFVector4& Vec2 )
{
	uniform WideFVector4 Temp1, Temp2;
	Temp1 = Vec1 * Vec2;
	Temp2 = VectorSwizzle(Temp1, 2,3,0,1);	// (Z,W,X,Y).
	Temp1 = Temp1 + Temp2; // (X*X + Z*Z, Y*Y + W*W, Z*Z + X*X, W*W + Y*Y)
	Temp2 = VectorSwizzle(Temp1, 1,2,3,0); // Rotate left 4 bytes (Y,Z,W,X).
	return Temp1 + Temp2; // (X*X + Z*Z + Y*Y + W*W, Y*Y + W*W + Z*Z + X*X, Z*Z + X*X + W*W + Y*Y, W*W + Y*Y + X*X + Z*Z)
}

unmasked inline uniform WideFVector4 VectorBitwiseAnd(const uniform WideFVector4 &Vec1, const uniform WideFVector4 &Vec2)
{
	varying FReal S0, S1, Result;
	*((uniform WideFVector4 *uniform)&S0) = *((uniform WideFVector4 *uniform)&Vec1);
	*((uniform WideFVector4 *uniform)&S1) = *((uniform WideFVector4 *uniform)&Vec2);

	Result = REAL_BITS(intbits(S0) & intbits(S1));

	return *((uniform WideFVector4 *uniform)&Result);
}

unmasked inline uniform WideFVector4 VectorBitwiseOr(const uniform WideFVector4 &Vec1, const uniform WideFVector4 &Vec2)
{
	varying FReal S0, S1, Result;
	*((uniform WideFVector4 *uniform)&S0) = *((uniform WideFVector4 *uniform)&Vec1);
	*((uniform WideFVector4 *uniform)&S1) = *((uniform WideFVector4 *uniform)&Vec2);

	Result = REAL_BITS(intbits(S0) | intbits(S1));

	return *((uniform WideFVector4 *uniform)&Result);
}

unmasked inline uniform WideFVector4 VectorBitwiseXor(const uniform WideFVector4 &Vec1, const uniform WideFVector4 &Vec2)
{
	varying FReal S0, S1, Result;
	*((uniform WideFVector4 *uniform)&S0) = *((uniform WideFVector4 *uniform)&Vec1);
	*((uniform WideFVector4 *uniform)&S1) = *((uniform WideFVector4 *uniform)&Vec2);

	Result = REAL_BITS(intbits(S0) ^ intbits(S1));

	return *((uniform WideFVector4 *uniform)&Result);
}

unmasked inline uniform WideFVector4 VectorSelect(const uniform WideFVector4& Mask, const uniform WideFVector4& Vec1, const uniform WideFVector4& Vec2 )
{
	return VectorBitwiseXor(Vec2, VectorBitwiseAnd(Mask, VectorBitwiseXor(Vec1, Vec2)));
}

unmasked inline uniform WideFVector4 VectorNormalizeSafe( const uniform WideFVector4& Vector, const uniform WideFVector4& DefaultValue)
{
	const uniform WideFVector4 SquareSum = VectorDot4(Vector, Vector);
	const uniform WideFVector4 NonZeroMask = VectorCompareGE(SquareSum, WSmallLengthThreshold);
	const uniform WideFVector4 InvLength = VectorReciprocalSqrtAccurate(SquareSum);
	const uniform WideFVector4 NormalizedVector = InvLength * Vector;
	return VectorSelect(NonZeroMask, NormalizedVector, DefaultValue);
}

unmasked inline uniform WideFVector4 VectorNormalizeQuaternion(const uniform WideFVector4& UnnormalizedQuat)
{
	return VectorNormalizeSafe(UnnormalizedQuat, WFloat0001);
}

unmasked inline uniform WideFVector4 VectorCross(const uniform WideFVector4& Vec1, const uniform WideFVector4& Vec2)
{
	const uniform WideFVector4 A_YZXW = VectorSwizzle(Vec1, 1,2,0,3);
	const uniform WideFVector4 B_ZXYW = VectorSwizzle(Vec2, 2,0,1,3);
	const uniform WideFVector4 A_ZXYW = VectorSwizzle(Vec1, 2,0,1,3);
	const uniform WideFVector4 B_YZXW = VectorSwizzle(Vec2, 1,2,0,3);
	return (A_YZXW * B_ZXYW) - (A_ZXYW * B_YZXW);
}

unmasked inline uniform WideFVector4 VectorQuaternionRotateVector(const uniform WideFVector4& Quat, const uniform WideFVector4& VectorW0)
{
	// Q * V * Q.Inverse
	//const VectorRegister InverseRotation = VectorQuaternionInverse(Quat);
	//const VectorRegister Temp = VectorQuaternionMultiply2(Quat, VectorW0);
	//const VectorRegister Rotated = VectorQuaternionMultiply2(Temp, InverseRotation);

	// Equivalence of above can be shown to be:
	// http://people.csail.mit.edu/bkph/articles/Quaternions.pdf
	// V' = V + 2w(Q x V) + (2Q x (Q x V))
	// refactor:
	// V' = V + w(2(Q x V)) + (Q x (2(Q x V)))
	// T = 2(Q x V);
	// V' = V + w*(T) + (Q x T)

	const uniform WideFVector4 QW = VectorReplicate(Quat, 3);
	uniform WideFVector4 T = VectorCross(Quat, VectorW0);
	T = T + T;
	const uniform WideFVector4 VTemp0 = QW * T + VectorW0;
	const uniform WideFVector4 VTemp1 = VectorCross(Quat, T);
	const uniform WideFVector4 Rotated = VTemp0 + VTemp1;
	return Rotated;
}

inline uniform WideFVector4 VectorAccumulateQuaternionShortestPath(const uniform WideFVector4& A, const uniform WideFVector4& B)
{
	// Blend rotation
	//     To ensure the 'shortest route', we make sure the dot product between the both rotations is positive.
	//     const float Bias = (|A.B| >= 0 ? 1 : -1)
	//     return A + B * Bias;
	const uniform WideFVector4 RotationDot = VectorDot4(A, B);
	const uniform WideFVector4 QuatRotationDirMask = VectorCompareGE(RotationDot, WFloatZero);
	const uniform WideFVector4 NegativeB = WFloatZero - B;
	const uniform WideFVector4 BiasTimesB = VectorSelect(QuatRotationDirMask, B, NegativeB);
	return A + BiasTimesB;
}

unmasked inline void LoadStridedWideFVector4(uniform FVector4 * uniform DstPtr, const uniform FVector4* uniform SrcPtr, const uniform int SrcStride)
{
	*DstPtr = *SrcPtr;
#if TARGET_WIDTH == 8 || TARGET_WIDTH == 16
	*(DstPtr + 1) = *(SrcPtr + SrcStride);
#endif
#if TARGET_WIDTH == 16
	*(DstPtr + 2) = *(SrcPtr + 2*SrcStride);
	*(DstPtr + 3) = *(SrcPtr + 3*SrcStride);
#endif
	/* Uses stack; use other method until fixed
	for(uniform int i = 0; i < (programCount / 4); i++)
	{
		*(DstPtr + i) = *(SrcPtr + i*SrcStride);
	}
	*/
}

unmasked inline void StoreStridedWideFVector4(uniform FVector4* uniform DstPtr, const uniform FVector4 * uniform SrcPtr, const uniform int DstStride)
{
	*DstPtr = *SrcPtr;
#if TARGET_WIDTH == 8 || TARGET_WIDTH == 16
	*(DstPtr + DstStride) = *(SrcPtr + 1);
#endif
#if TARGET_WIDTH == 16
	*(DstPtr + 2*DstStride) = *(SrcPtr + 2);
	*(DstPtr + 3*DstStride) = *(SrcPtr + 3);
#endif
	/* Uses stack; use other method until fixed
	for(uniform int i = 0; i < (programCount / 4); i++)
	{
		*(DstPtr + i*DstStride) = *(SrcPtr + i);
	}
	*/
}

unmasked inline void LoadIndexedWideFVector4(uniform FVector4 * uniform DstPtr, const uniform FVector4* uniform SrcPtr, const uniform int* uniform pIndex)
{
	*DstPtr = *(SrcPtr + pIndex[0]);
#if TARGET_WIDTH == 8 || TARGET_WIDTH == 16
	*(DstPtr + 1) = *(SrcPtr + pIndex[1]);
#endif
#if TARGET_WIDTH == 16
	*(DstPtr + 2) = *(SrcPtr + pIndex[2]);
	*(DstPtr + 3) = *(SrcPtr + pIndex[3]);
#endif
}

unmasked inline void StoreIndexedWideFVector4(uniform FVector4 * uniform DstPtr, const uniform FVector4* uniform SrcPtr, const uniform int* uniform pIndex)
{
	*(DstPtr + pIndex[0]) = *SrcPtr;
#if TARGET_WIDTH == 8 || TARGET_WIDTH == 16
	*(DstPtr + pIndex[1]) = *(SrcPtr + 1);
#endif
#if TARGET_WIDTH == 16
	*(DstPtr + pIndex[2]) = *(SrcPtr + 2);
	*(DstPtr + pIndex[3]) = *(SrcPtr + 3);
#endif
}

#endif
