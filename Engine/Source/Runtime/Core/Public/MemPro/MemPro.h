/*
Copyright 2019 PureDev Software Limited

Permission to use, copy, modify, and/or distribute this software for any
purpose with or without fee is hereby granted, provided that the above
copyright notice and this permission notice appear in all copies.

THE SOFTWARE IS PROVIDED "AS IS" AND THE AUTHOR DISCLAIMS ALL WARRANTIES
WITH REGARD TO THIS SOFTWARE INCLUDING ALL IMPLIED WARRANTIES OF
MERCHANTABILITY AND FITNESS. IN NO EVENT SHALL THE AUTHOR BE LIABLE FOR
ANY SPECIAL, DIRECT, INDIRECT, OR CONSEQUENTIAL DAMAGES OR ANY DAMAGES
WHATSOEVER RESULTING FROM LOSS OF USE, DATA OR PROFITS, WHETHER IN AN
ACTION OF CONTRACT, NEGLIGENCE OR OTHER TORTIOUS ACTION, ARISING OUT OF
OR IN CONNECTION WITH THE USE OR PERFORMANCE OF THIS SOFTWARE.
*/


//------------------------------------------------------------------------
//
// MemPro.h
//
/*
	MemProLib is the library that allows the MemPro application to communicate
	with your application.

	===========================================================
                             SETUP
	===========================================================

	* include MemPro.cpp and MemPro.h in your project.

	* Link with Dbghelp.lib and Ws2_32.lib - these are needed for the callstack trace and the network connection

	* Connect to your app with the MemPro
*/
//------------------------------------------------------------------------
/*
	MemPro
	Version:	1.6.3.0
*/
//------------------------------------------------------------------------
#ifndef MEMPRO_MEMPRO_H_INCLUDED
#define MEMPRO_MEMPRO_H_INCLUDED

//------------------------------------------------------------------------
//@EPIC BEGIN - enabled in build config not here
#if !defined(__UNREAL__)
	#define MEMPRO_ENABLED 1				// **** enable/disable MemPro here! ****
#elif !defined(MEMPRO_ENABLED)
	#define MEMPRO_ENABLED 0
#endif
//@EPIC END

//------------------------------------------------------------------------
#if defined(__UNREAL__) && MEMPRO_ENABLED && !defined(WITH_ENGINE)
	#undef MEMPRO_ENABLED
	#define MEMPRO_ENABLED 0
#endif

//------------------------------------------------------------------------
#if defined(__UNREAL__)
	#include "CoreTypes.h"
	#include "HAL/PlatformMisc.h"
#endif

//------------------------------------------------------------------------
// **** The Target Platform ****

// define ONE of these
//@EPIC BEGIN external definition of platforms
#if defined(MEMPRO_PLATFORM_XBOXONE)
#elif defined(MEMPRO_PLATFORM_PS4)
#elif defined(MEMPRO_PLATFORM_EXTENSION)
#elif defined(_WIN32) || defined(_WIN64) || defined(WIN32) || defined(WIN64) || defined(__WIN32__)
//@EPIC END
	#if defined(_XBOX_ONE)
		#define MEMPRO_PLATFORM_XBOXONE
	#elif defined(_XBOX)
		#define MEMPRO_PLATFORM_XBOX360
	#else
		#define MEMPRO_PLATFORM_WIN
	#endif
#elif defined(__APPLE__)
	#define MEMPRO_PLATFORM_APPLE
#else
	#define MEMPRO_PLATFORM_UNIX
#endif

//------------------------------------------------------------------------
// macros for tracking allocs that define to nothing if disabled
#if MEMPRO_ENABLED
	#ifndef WAIT_FOR_CONNECT
		#define WAIT_FOR_CONNECT false
	#endif
	#define MEMPRO_TRACK_ALLOC(p, size) MemPro::TrackAlloc(p, size, WAIT_FOR_CONNECT)
	#define MEMPRO_TRACK_FREE(p) MemPro::TrackFree(p, WAIT_FOR_CONNECT)
#else
	#define MEMPRO_TRACK_ALLOC(p, size) ((void)0)
	#define MEMPRO_TRACK_FREE(p) ((void)0)
#endif

//------------------------------------------------------------------------
#if MEMPRO_ENABLED

//------------------------------------------------------------------------
// Some platforms have problems initialising winsock from global constructors,
// to help get around this problem MemPro waits this amount of time before
// initialising. Allocs and freed that happen during this time are stored in
// a temporary buffer.
#define MEMPRO_INIT_DELAY 100

//------------------------------------------------------------------------
// MemPro waits this long before giving up on a connection after initialisation
#define MEMPRO_CONNECT_TIMEOUT 500

//------------------------------------------------------------------------
#include <stdlib.h>

//------------------------------------------------------------------------
//#define MEMPRO_WRITE_DUMP "d:\\temp\\allocs.mempro_dump"

//------------------------------------------------------------------------
// always default to use dump files for Unreal
#if defined(__UNREAL__) && !defined(MEMPRO_WRITE_DUMP)
	#define MEMPRO_WRITE_DUMP
#endif

//------------------------------------------------------------------------
#if defined(MEMPRO_WRITE_DUMP) && defined(DISALLOW_WRITE_DUMP) 
	#error
#endif

//------------------------------------------------------------------------
#define MEMPRO_ASSERT(b) if(!(b)) Platform::DebugBreak()

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	typedef long long int64;
	typedef unsigned long long uint64;

	//------------------------------------------------------------------------
	enum PageState
	{
		Invalid = -1,
		Free,
		Reserved,
		Committed
	};

	//------------------------------------------------------------------------
	enum PageType
	{
		page_Unknown = -1,
		page_Image,
		page_Mapped,
		page_Private
	};

	//------------------------------------------------------------------------
	enum EPlatform
	{
		Platform_Windows,
		Platform_Unix,
		Platform_PS4,
	};

	//------------------------------------------------------------------------
	typedef int(*ThreadMain)(void*);

	//------------------------------------------------------------------------
	typedef void(*SendPageStateFunction)(void*, size_t, PageState, PageType, unsigned int, bool, int, void*);

	//------------------------------------------------------------------------
	typedef void(*EnumerateLoadedModulesCallbackFunction)(int64, const char*, void*);

	//------------------------------------------------------------------------
	// You don't need to call this directly, it is automatically called on the first allocation.
	// Only call this function if you want to be able to connect to your app before it has allocated any memory.
	// If wait_for_connect is true this function will block until the external MemPro app has connected,
	// this is useful to make sure that every single allocation is being tracked.
	void Initialise(bool wait_for_connect=false);

	void Disconnect();		// kick all current connections, but can accept more

	void Shutdown();		// free all resources, no more connections allowed

	void TrackAlloc(void* p, size_t size, bool wait_for_connect=false);

	void TrackFree(void* p, bool wait_for_connect=false);

	bool IsPaused();

	void SetPaused(bool paused);

	void TakeSnapshot(bool send_memory=false);

	void FlushDumpFile();

	// ignore these, for internal use only
	void IncRef();
	void DecRef();
}

//------------------------------------------------------------------------
#ifndef MEMPRO_WRITE_DUMP
namespace
{
	// if we are using sockets we need to flush the sockets on global teardown
	// This class is a trick to attempt to get mempro to shutdown after all other
	// global objects.
	class MemProGLobalScope
	{
	public:
		MemProGLobalScope() { MemPro::IncRef(); }
		~MemProGLobalScope() { MemPro::DecRef(); }
	};
	static MemProGLobalScope g_MemProGLobalScope;
}
#endif

//------------------------------------------------------------------------

//------------------------------------------------------------------------
//
// MemProPlatform.hpp
//
//------------------------------------------------------------------------
#if MEMPRO_ENABLED

//------------------------------------------------------------------------
#if defined(MEMPRO_PLATFORM_WIN)

	#define MEMPRO_WIN_BASED_PLATFORM
	#define MEMPRO_INTERLOCKED_ALIGN __declspec(align(8))
	#define MEMPRO_INSTRUCTION_BARRIER
	#define MEMPRO_ENABLE_WARNING_PRAGMAS
	#define MEMPRO_PUSH_WARNING_DISABLE __pragma(warning(push))
	#define MEMPRO_DISABLE_WARNING(w) __pragma(warning(disable : w))
	#define MEMPRO_POP_WARNING_DISABLE __pragma(warning(pop))
	#define MEMPRO_FORCEINLINE FORCEINLINE
	#define ENUMERATE_ALL_MODULES // if you are having problems compiling this on your platform undefine ENUMERATE_ALL_MODULES and it send info for just the main module
	#define THREAD_LOCAL_STORAGE __declspec(thread)
	#define MEMPRO_PORT "27016"
	#define STACK_TRACE_SIZE 128
	#define MEMPRO_PAGE_SIZE 4096
	//#define USE_RTLCAPTURESTACKBACKTRACE
	#define MEMPRO_ALIGN_SUFFIX(n)

	#if defined(_WIN64) || defined(__LP64__) || defined(__x86_64__) || defined(__ppc64__)
		#define MEMPRO64
	#endif

	#ifdef MEMPRO64
		#define MEMPRO_MAX_ADDRESS ULLONG_MAX
	#else
		#define MEMPRO_MAX_ADDRESS UINT_MAX
	#endif

	#ifdef __UNREAL__
		#include "Windows/AllowWindowsPlatformTypes.h"
	#endif

	#ifndef MEMPRO_WRITE_DUMP
		#if defined(UNICODE) && !defined(_UNICODE)
			#error for unicode builds please define both UNICODE and _UNICODE. See the FAQ for more details.
		#endif
		#if defined(AF_IPX) && !defined(_WINSOCK2API_)
			#error winsock already defined. Please include winsock2.h before including windows.h or use WIN32_LEAN_AND_MEAN. See the FAQ for more info.
		#endif

		#pragma warning(push)
		#pragma warning(disable : 4668)
		#include <winsock2.h>
		#pragma warning(pop)

		#include <ws2tcpip.h>
		#ifndef _WIN32_WINNT
			#define _WIN32_WINNT 0x0501
		#endif						
	#endif

	#define WINDOWS_LEAN_AND_MEAN
	#include <windows.h>
	#include <intrin.h>

	#ifdef ENUMERATE_ALL_MODULES
		#pragma warning(push)
		#pragma warning(disable : 4091)
		#include <Dbghelp.h>
		#pragma warning(pop)
		#pragma comment(lib, "Dbghelp.lib")
	#endif

	#ifdef __UNREAL__
		#include "Windows/HideWindowsPlatformTypes.h"
	#endif

//@EPIC BEGIN: other platforms
#elif defined(MEMPRO_PLATFORM_EXTENSION) && defined(__UNREAL__)

	#include COMPILED_PLATFORM_HEADER_WITH_PREFIX(MemPro,MemProExt.h)

//@EPIC END

#elif defined(MEMPRO_PLATFORM_XBOXONE)

	#ifdef __UNREAL__
		#include "MemPro/MemProXboxOne.h"
	#else
		#include "MemProXboxOne.hpp"		// contact slynch@puredevsoftware.com for this platform
	#endif

#elif defined(MEMPRO_PLATFORM_XBOX360)

	#include "MemProXbox360.hpp"		// contact slynch@puredevsoftware.com for this platform

#elif defined(MEMPRO_PLATFORM_PS4)

	#ifdef __UNREAL__
		#include "MemPro/MemProPS4.h"
	#else
		#include "MemProPS4.hpp"			// contact slynch@puredevsoftware.com for this platform
	#endif

#elif defined(MEMPRO_PLATFORM_UNIX)

	#define MEMPRO_UNIX_BASED_PLATFORM
	#define MEMPRO_INTERLOCKED_ALIGN
	#define MEMPRO_INSTRUCTION_BARRIER
	#define MEMPRO_PUSH_WARNING_DISABLE
	#define MEMPRO_DISABLE_WARNING(w)
	#define MEMPRO_POP_WARNING_DISABLE
	#define MEMPRO_FORCEINLINE inline
	#define ENUMERATE_ALL_MODULES
	#define THREAD_LOCAL_STORAGE __thread
	#define MEMPRO_PORT "27016"
	#define STACK_TRACE_SIZE 128
	#define MEMPRO_ALIGN_SUFFIX(n) __attribute__ ((aligned(n)))

	#if defined(__LP64__) || defined(__x86_64__) || defined(__ppc64__)
		#define MEMPRO64
	#endif

#elif defined(MEMPRO_PLATFORM_APPLE)

	#define MEMPRO_UNIX_BASED_PLATFORM
	#define MEMPRO_INTERLOCKED_ALIGN
	#define MEMPRO_INSTRUCTION_BARRIER
	#define MEMPRO_PUSH_WARNING_DISABLE
	#define MEMPRO_DISABLE_WARNING(w)
	#define MEMPRO_POP_WARNING_DISABLE
	#define MEMPRO_FORCEINLINE inline
	#define ENUMERATE_ALL_MODULES
	#define THREAD_LOCAL_STORAGE __thread
	#define MEMPRO_PORT "27016"
	#define STACK_TRACE_SIZE 128
	#define MEMPRO_ALIGN_SUFFIX(n) __attribute__ ((aligned(n)))

	#if defined(__LP64__) || defined(__x86_64__) || defined(__ppc64__)
		#define MEMPRO64
	#endif

	#ifdef OVERRIDE_NEW_DELETE
		// if you get linker errors about duplicatly defined symbols please add a unexport.txt
		// file to your build settings
		// see here: https://developer.apple.com/library/mac/technotes/tn2185/_index.html
		void* operator new(std::size_t size) throw(std::bad_alloc)
		{
			void* p = malloc(size);
			MEMPRO_TRACK_ALLOC(p, size);
			return p;
		}

		void* operator new(std::size_t size, const std::nothrow_t&) throw()
		{
			void* p = malloc(size);
			MEMPRO_TRACK_ALLOC(p, size);
			return p;
		}

		void  operator delete(void* p) throw()
		{
			MEMPRO_TRACK_FREE(p);
			free(p);
		}

		void  operator delete(void* p, const std::nothrow_t&) throw()
		{
			MEMPRO_TRACK_FREE(p);
			free(p);
		}

		void* operator new[](std::size_t size) throw(std::bad_alloc)
		{
			void* p = malloc(size);
			MEMPRO_TRACK_ALLOC(p, size);
			return p;
		}

		void* operator new[](std::size_t size, const std::nothrow_t&) throw()
		{
			void* p = malloc(size);
			MEMPRO_TRACK_ALLOC(p, size);
			return p;
		}

		void  operator delete[](void* p) throw()
		{
			MEMPRO_TRACK_FREE(p);
			free(p);
		}

		void  operator delete[](void* p, const std::nothrow_t&) throw()
		{
			MEMPRO_TRACK_FREE(p);
			free(p);
		}
	#endif

#else

	#error

#endif

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	namespace Platform
	{
		void CreateLock(void* p_os_lock_mem, int os_lock_mem_size);

		void DestroyLock(void* p_os_lock_mem);

		void TakeLock(void* p_os_lock_mem);

		void ReleaseLock(void* p_os_lock_mem);

		//

#ifndef MEMPRO_WRITE_DUMP
		void UninitialiseSockets();

		void CreateSocket(void* p_os_socket_mem, int os_socket_mem_size);

		bool IsValidSocket(const void* p_os_socket_mem);

		void Disconnect(void* p_os_socket_mem);

		bool StartListening(void* p_os_socket_mem);

		bool BindSocket(void* p_os_socket_mem, const char* p_port);
		
		bool AcceptSocket(void* p_os_socket_mem, void* p_client_os_socket_mem);

		bool SocketSend(void* p_os_socket_mem, void* p_buffer, int size);
		
		int SocketReceive(void* p_os_socket_mem, void* p_buffer, int size);
#endif
		//

		void MemProCreateEvent(
			void* p_os_event_mem,
			int os_event_mem_size,
			bool initial_state,
			bool auto_reset);

		void DestroyEvent(void* p_os_event_mem);

		void SetEvent(void* p_os_event_mem);

		void ResetEvent(void* p_os_event_mem);

		int WaitEvent(void* p_os_event_mem, int timeout);

		//

		void CreateThread(void* p_os_thread_mem, int os_thread_mem_size);

		void DestroyThread(void* p_os_thread_mem);

		int StartThread(void* p_os_thread_mem, ThreadMain p_thread_main, void* p_param);

		bool IsThreadAlive(const void* p_os_thread_mem);

		//

		int64 MemProInterlockedCompareExchange(int64 volatile *dest, int64 exchange, int64 comperand);

		int64 MemProInterlockedExchangeAdd(int64 volatile *Addend, int64 Value);

		void SwapEndian(unsigned int& value);

		void SwapEndian(uint64& value);

		void DebugBreak();

		void* Alloc(int size);

		void Free(void* p, int size);

		int64 GetHiResTimer();

		int64 GetHiResTimerFrequency();

		void SetThreadName(unsigned int thread_id, const char* p_name);

		void Sleep(int ms);

		void GetStackTrace(void** stack, int& stack_size, unsigned int& hash);

		void SendPageState(
			bool send_memory,
			SendPageStateFunction send_page_state_function,
			void* p_context);

		void GetVirtualMemStats(size_t& reserved, size_t& committed);

		bool GetExtraModuleInfo(
			int64 ModuleBase,
			int& age,
			void* p_guid,
			int guid_size,
			char* p_pdb_filename,
			int pdb_filename_size);

		void MemProEnumerateLoadedModules(
			EnumerateLoadedModulesCallbackFunction p_callback_function,
			void* p_context);

		void DebugWrite(const char* p_message);

		void MemProMemoryBarrier();

		EPlatform GetPlatform();

		int GetStackTraceSize();

		void MemCpy(void* p_dest, int dest_size, const void* p_source, int source_size);

		void SPrintF(char* p_dest, int dest_size, const char* p_format, const char* p_str);

		//

		void MemProCreateFile(void* p_os_file_mem, int os_file_mem_size);

		void DestroyFile(void* p_os_file_mem);

		bool OpenFileForWrite(void* p_os_file_mem, const char* p_filename);

		void CloseFile(void* p_os_file_mem);

		void FlushFile(void* p_os_file_mem);

		bool WriteFile(void* p_os_file_mem, const void* p_data, int size);

#ifdef MEMPRO_WRITE_DUMP
		void GetDumpFilename(char* p_filename, int max_length);
#endif
	}
}

//------------------------------------------------------------------------
namespace MemPro
{
	//------------------------------------------------------------------------
	namespace GenericPlatform
	{
		void CreateLock(void* p_os_lock_mem, int os_lock_mem_size);

		void DestroyLock(void* p_os_lock_mem);

		void TakeLock(void* p_os_lock_mem);

		void ReleaseLock(void* p_os_lock_mem);

#ifndef MEMPRO_WRITE_DUMP
		bool InitialiseSockets();

		void UninitialiseSockets();

		void CreateSocket(void* p_os_socket_mem, int os_socket_mem_size);

		bool IsValidSocket(const void* p_os_socket_mem);

		void Disconnect(void* p_os_socket_mem);

		bool StartListening(void* p_os_socket_mem);

		bool BindSocket(void* p_os_socket_mem, const char* p_port);

		bool AcceptSocket(void* p_os_socket_mem, void* p_client_os_socket_mem);

		bool SocketSend(void* p_os_socket_mem, void* p_buffer, int size);

		int SocketReceive(void* p_os_socket_mem, void* p_buffer, int size);
#endif
		void MemProCreateEvent(
			void* p_os_event_mem,
			int os_event_mem_size,
			bool initial_state,
			bool auto_reset);

		void DestroyEvent(void* p_os_event_mem);

		void SetEvent(void* p_os_event_mem);

		void ResetEvent(void* p_os_event_mem);

		int WaitEvent(void* p_os_event_mem, int timeout);

		void CreateThread(void* p_os_thread_mem, int os_thread_mem_size);

		void DestroyThread(void* p_os_thread_mem);

		int StartThread(void* p_os_thread_mem, ThreadMain p_thread_main, void* p_param);

		bool IsThreadAlive(const void* p_os_thread_mem);

		int64 MemProInterlockedCompareExchange(int64 volatile *dest, int64 exchange, int64 comperand);

		int64 MemProInterlockedExchangeAdd(int64 volatile *Addend, int64 Value);

		void SwapEndian(unsigned int& value);

		void SwapEndian(uint64& value);

		void DebugBreak();

		void* Alloc(int size);

		void Free(void* p, int size);

		void SetThreadName(unsigned int thread_id, const char* p_name);

		void Sleep(int ms);

		void SendPageState(
			bool send_memory,
			SendPageStateFunction send_page_state_function,
			void* p_context);

		void GetVirtualMemStats(size_t& reserved, size_t& committed);

		bool GetExtraModuleInfo(
			int64 ModuleBase,
			int& age,
			void* p_guid,
			int guid_size,
			char* p_pdb_filename,
			int pdb_filename_size);

		void MemProEnumerateLoadedModules(EnumerateLoadedModulesCallbackFunction p_callback_function, void* p_context);

		void DebugWrite(const char* p_message);

		void MemCpy(void* p_dest, int dest_size, const void* p_source, int source_size);

		void SPrintF(char* p_dest, int dest_size, const char* p_format, const char* p_str);

		void MemProCreateFile(void* p_os_file_mem, int os_file_mem_size);

		void DestroyFile(void* p_os_file_mem);

		bool OpenFileForWrite(void* p_os_file_mem, const char* p_filename);

		void CloseFile(void* p_os_file_mem);

		void FlushFile(void* p_os_file_mem);

		bool WriteFile(void* p_os_file_mem, const void* p_data, int size);

		#ifdef MEMPRO_WRITE_DUMP
		void GetDumpFilename(char* p_filename, int max_length);
		#endif
	}
}

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED

//------------------------------------------------------------------------
#ifdef OVERRIDE_NEW_DELETE

	#if defined(_WIN32) || defined(_WIN64) || defined(WIN32) || defined(WIN64) || defined(__WIN32__) || defined(__WINDOWS__)
		#include <malloc.h>

		void* operator new(size_t size)
		{
			void* p = malloc(size);
			MEMPRO_TRACK_ALLOC(p, size);
			return p;
		}

		void operator delete(void* p)
		{
			MEMPRO_TRACK_FREE(p);
			free(p);
		}

		void* operator new[](size_t size)
		{
			void* p = malloc(size);
			MEMPRO_TRACK_ALLOC(p, size);
			return p;
		}

		void operator delete[](void* p)
		{
			MEMPRO_TRACK_FREE(p);
			free(p);
		}
	#endif

#endif

//------------------------------------------------------------------------
#ifdef OVERRIDE_MALLOC_FREE
	
	#if defined(_WIN32) || defined(_WIN64) || defined(WIN32) || defined(WIN64) || defined(__WIN32__) || defined(__WINDOWS__)
		
		// NOTE: for this to work, you will need to make sure you are linking STATICALLY to the crt. eg: /MTd

		__declspec(restrict) __declspec(noalias) void* malloc(size_t size)
		{
			void* p = HeapAlloc(GetProcessHeap(), 0, size);
			MEMPRO_TRACK_ALLOC(p, size);
			return p;
		}

		__declspec(restrict) __declspec(noalias) void* realloc(void *p, size_t new_size)
		{
			MEMPRO_TRACK_FREE(p);
			void* p_new = HeapReAlloc(GetProcessHeap(), 0, p, new_size);
			MEMPRO_TRACK_ALLOC(p_new, new_size);
			return p_new;
		}

		__declspec(noalias) void free(void *p)
		{
			HeapFree(GetProcessHeap(), 0, p);
			MEMPRO_TRACK_FREE(p);
		}
	#else
		void *malloc(int size)
		{
			void* (*ptr)(int);
			void* handle = (void*)-1;
			ptr = (void*)dlsym(handle, "malloc");
			if(!ptr) abort();
			void *p = (*ptr)(size);
			MEMPRO_TRACK_ALLOC(p, size);
			return p;
		}

		void *realloc(void *p, int size)
		{
			MEMPRO_TRACK_FREE(p);
			void * (*ptr)(void *, int);
			void * handle = (void*) -1;
			ptr = (void*)dlsym(handle, "realloc");
			if (!ptr) abort();
			void* p_new = (*ptr)(p, size);
			MEMPRO_TRACK_ALLOC(p_new, size);
			return p_new;
		}

		void free(void *p)
		{
			MEMPRO_TRACK_FREE(p);
			void* (*ptr)(void*);
			void* handle = (void*)-1;
			ptr = (void*)dlsym(handle, "free");
			if (!ptr == NULL) abort();
			(*ptr)(alloc);
		}
	#endif
#endif

//------------------------------------------------------------------------
#endif		// #if MEMPRO_ENABLED

//------------------------------------------------------------------------
#endif		// #ifndef MEMPRO_MEMPRO_H_INCLUDED

