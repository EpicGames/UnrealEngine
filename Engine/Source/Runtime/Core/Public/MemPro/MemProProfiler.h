// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* 
 * To enable MemPro support in the engine, add GlobalDefinitions.Add("MEMPRO_ENABLED=1"); to your game's .Target.cs build rules file for supported platforms.
 * (note: MemPro.cpp/.h need to be added to the project for this to work)
 */
#if !defined(MEMPRO_ENABLED)
	#define MEMPRO_ENABLED 0		
#endif



#if MEMPRO_ENABLED
#include "CoreGlobals.h"
#include "HAL/LowLevelMemTracker.h"
#include "MemPro/MemPro.h"
#include "Containers/StaticArray.h"

class FMemProProfiler
{
public:
	static CORE_API void Init(const TCHAR* CmdLine);

	static CORE_API bool IsUsingPort( uint32 Port );

	static inline bool IsStarted()
	{
		extern int32 GMemProEnabled;
		return (GMemProEnabled != 0) && !IsEngineExitRequested();
	}

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	static inline bool IsTrackingTag( ELLMTag Tag )
	{
		extern TStaticArray<bool,LLM_TAG_COUNT> MemProLLMTagsEnabled;
		return IsStarted() && MemProLLMTagsEnabled[(int32)Tag];
	}

	static CORE_API void TrackTag( ELLMTag Tag );
	static CORE_API void TrackTagsByName( const TCHAR* TagNamesStr );
#endif //ENABLE_LOW_LEVEL_MEM_TRACKER
};

#endif //MEMPRO_ENABLED
