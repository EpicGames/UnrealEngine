// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "HAL/Platform.h"
#include "Math/NumericLimits.h"
#include "Memory/MemoryFwd.h"
#include "Memory/MemoryView.h"
#include "Memory/SharedBuffer.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/FunctionFwd.h"

#include <type_traits>

/**
 * FCompositeBuffer is a non-contiguous buffer composed of zero or more immutable shared buffers.
 *
 * A composite buffer is most efficient when its segments are consumed as they are, but it can be
 * flattened into a contiguous buffer when necessary, by calling ToShared(). Ownership of segment
 * buffers is not changed on construction, but if ownership of segments is required then that can
 * be guaranteed by calling MakeOwned().
 */
class FCompositeBuffer
{
	template <typename BufferType>
	using CanAppendBufferType = std::bool_constant<
		std::is_same_v<FCompositeBuffer, BufferType> ||
		std::is_same_v<FSharedBuffer, BufferType> ||
		TIsTArray_V<BufferType>>;

public:
	/**
	 * Construct a composite buffer by concatenating the buffers. Does not enforce ownership.
	 *
	 * Buffer parameters may be FSharedBuffer, FCompositeBuffer, or TArray<FSharedBuffer>.
	 */
	template <typename... BufferTypes,
		std::enable_if_t<std::conjunction_v<CanAppendBufferType<std::decay_t<BufferTypes>>...>>* = nullptr>
	inline explicit FCompositeBuffer(BufferTypes&&... Buffers)
	{
		if constexpr (sizeof...(Buffers) > 0)
		{
			Segments.Reserve((GetBufferCount(Forward<BufferTypes>(Buffers)) + ...));
			(AppendBuffers(Forward<BufferTypes>(Buffers)), ...);
			Segments.RemoveAll(&FSharedBuffer::IsNull);
		}
	}

	/** Destructor. */
	CORE_API ~FCompositeBuffer();

	/** Reset this to null. */
	CORE_API void Reset();

	/** Returns the total size of the composite buffer in bytes. */
	[[nodiscard]] CORE_API uint64 GetSize() const;

	/** Returns the segments that the buffer is composed from. */
	[[nodiscard]] inline TConstArrayView<FSharedBuffer> GetSegments() const { return Segments; }

	/** Returns true if the composite buffer is not null. */
	[[nodiscard]] inline explicit operator bool() const { return !IsNull(); }

	/** Returns true if the composite buffer is null. */
	[[nodiscard]] inline bool IsNull() const { return Segments.IsEmpty(); }

	/** Returns true if every segment in the composite buffer is owned. */
	[[nodiscard]] CORE_API bool IsOwned() const;

	/** Returns a copy of the buffer where every segment is owned. */
	[[nodiscard]] CORE_API FCompositeBuffer MakeOwned() const &;
	[[nodiscard]] CORE_API FCompositeBuffer MakeOwned() &&;

	/** Returns the concatenation of the segments into a contiguous buffer. */
	[[nodiscard]] CORE_API FSharedBuffer ToShared() const &;
	[[nodiscard]] CORE_API FSharedBuffer ToShared() &&;

	/** Returns the middle part of the buffer by taking the size starting at the offset. */
	[[nodiscard]] CORE_API FCompositeBuffer Mid(uint64 Offset, uint64 Size = MAX_uint64) const;

	/**
	 * Returns a view of the range if contained by one segment, otherwise a view of a copy of the range.
	 *
	 * @note CopyBuffer is reused if large enough, and otherwise allocated when needed.
	 *
	 * @param Offset       The byte offset in this buffer that the range starts at.
	 * @param Size         The number of bytes in the range to view or copy.
	 * @param CopyBuffer   The buffer to write the copy into if a copy is required.
	 * @param Allocator    The optional allocator to use when the copy buffer is required.
	 */
	[[nodiscard]] CORE_API FMemoryView ViewOrCopyRange(
		uint64 Offset,
		uint64 Size,
		FUniqueBuffer& CopyBuffer) const;
	[[nodiscard]] CORE_API FMemoryView ViewOrCopyRange(
		uint64 Offset,
		uint64 Size,
		FUniqueBuffer& CopyBuffer,
		TFunctionRef<FUniqueBuffer (uint64 Size)> Allocator) const;

	/**
	 * Copies a range of the buffer to a contiguous region of memory.
	 *
	 * @param Target   The view to copy to. Must be no larger than the data available at the offset.
	 * @param Offset   The byte offset in this buffer to start copying from.
	 */
	CORE_API void CopyTo(FMutableMemoryView Target, uint64 Offset = 0) const;

	/**
	 * Invokes a visitor with a view of each segment that intersects with a range.
	 *
	 * @param Offset    The byte offset in this buffer to start visiting from.
	 * @param Size      The number of bytes in the range to visit.
	 * @param Visitor   The visitor to invoke from zero to GetSegments().Num() times.
	 */
	CORE_API void IterateRange(uint64 Offset, uint64 Size,
		TFunctionRef<void (FMemoryView View)> Visitor) const;
	CORE_API void IterateRange(uint64 Offset, uint64 Size,
		TFunctionRef<void (FMemoryView View, const FSharedBuffer& ViewOuter)> Visitor) const;

	/** Returns true if the bytes of this buffer are equal to the bytes of the other buffer. */
	[[nodiscard]] CORE_API bool EqualBytes(const FCompositeBuffer& Other) const;

	/** A null composite buffer. */
	CORE_API static const FCompositeBuffer Null;

private:
	static inline int32 GetBufferCount(const FCompositeBuffer& Buffer) { return Buffer.Segments.Num(); }
	inline void AppendBuffers(const FCompositeBuffer& Buffer) { Segments.Append(Buffer.Segments); }
	inline void AppendBuffers(FCompositeBuffer&& Buffer) { Segments.Append(MoveTemp(Buffer.Segments)); }

	static inline int32 GetBufferCount(const FSharedBuffer& Buffer) { return 1; }
	inline void AppendBuffers(const FSharedBuffer& Buffer) { Segments.Add(Buffer); }
	inline void AppendBuffers(FSharedBuffer&& Buffer) { Segments.Add(MoveTemp(Buffer)); }

	template <typename BufferType, decltype(std::declval<TArray<FSharedBuffer>>().Append(std::declval<BufferType>()))* = nullptr>
	static inline int32 GetBufferCount(BufferType&& Buffer) { return GetNum(Buffer); }
	template <typename BufferType, decltype(std::declval<TArray<FSharedBuffer>>().Append(std::declval<BufferType>()))* = nullptr>
	inline void AppendBuffers(BufferType&& Buffer) { Segments.Append(Forward<BufferType>(Buffer)); }

private:
	TArray<FSharedBuffer, TInlineAllocator<1>> Segments;
};

