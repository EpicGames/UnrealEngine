// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HAL/UnrealMemory.h"
#include "HAL/CriticalSection.h"
#include "Templates/SharedPointerFwd.h" // ESPMode
#include "Templates/UnrealTemplate.h"

#include <atomic>

#ifndef UE_ENABLE_LINEAR_VIRTUAL_ALLOCATOR
#	define UE_ENABLE_LINEAR_VIRTUAL_ALLOCATOR PLATFORM_HAS_FPlatformVirtualMemoryBlock
#endif

namespace UE
{

class FLinearBlockAllocatorThreadAccessor;

/**
 * Provides Malloc operation that reduces per-malloc and per-free overhead by allocating in blocks, and not allowing
 * individual calls to free. Does not free any malloc'd memory until the allocator goes out of scope.
 * Intended use is dynamically allocated scratch data, or long-lived global allocations that never need to free data.
 * Allocations at the end of each block waste memory: when the next requested allocation is too big for the current
 * block, the remaining memory goes unused. To avoid large amounts of waste, allocations should be relatively small
 * compared to blocksize.
 *
 * Memory for the blocks is allocated from InnerMalloc->Malloc. If InnerMalloc is nullptr, GMalloc is used.
 * 
 * The LinearBlockAllocator must be destroyed before its InnerMalloc is destroyed; the InnerMalloc pointer is
 * not reference counted.
 */
class FLinearBlockAllocator : public FNoncopyable
{
public:
	/** 
	 * Constructor.
	 * 
	 * @param InInnerMalloc The allocator used to allocate blocks and internal memory. If null, uses GMalloc.
	 * @param InBlockSize The size of each block. Rounded up to MinBlockSize and next power of 2. If 0, uses
	 *        UE::LinearAllocator::DefaultBlockSize.
	 * @param InThreadMode If ESPMode::ThreadSafe, the allocator locks a critical section around each allocation
	 *        and can be used from any thread. If ESPMode::NotThreadSafe, the allocator must either be used only
	 *        from a single thread, or cannot be accessed directly and must instead be accessed through
	 *        FLinearBlockAllocatorThreadAccessor. Each FLinearBlockAllocatorThreadAccessor owns its own block.
	 *        The unallocated memory in each threaded block is wasted and not available to other threads.
	 */
	CORE_API FLinearBlockAllocator(FMalloc* InInnerMalloc = nullptr, uint32 InBlockSize = 0,
		ESPMode InThreadMode = ESPMode::ThreadSafe);
	CORE_API ~FLinearBlockAllocator();
	FLinearBlockAllocator(FLinearBlockAllocator&&) = delete;
	FLinearBlockAllocator& operator=(FLinearBlockAllocator&&) = delete;

	CORE_API void* Malloc(SIZE_T Size, uint32 Alignment = DEFAULT_ALIGNMENT);

	/** Returns the size of allocated blocks, which is >= size of pointers returned from Malloc calls. */
	CORE_API SIZE_T GetAllocatedMemorySize() const;

protected:
	struct FBlock
	{
		FBlock* Next; // Guarded by Lock
		uint32 BlockSize; // Constant during threading
		uint32 NextOffset; // Read/Written by owning thread or guarded by Lock in ESPMode::ThreadSafe

		static FBlock* GetHeaderFromBlockStart(void* Block, uint32 BlockSize);
		void* GetBlockStart();
	};
	constexpr static uint32 MinBlockSize = (uint32)(2 * sizeof(FBlock));

	bool IsThreadSafe() const;
	void* TryAllocateFromBlock(SIZE_T Size, uint32 Alignment, FBlock* BlockHeader);
	bool RequiresCustomAllocation(SIZE_T Size, uint32 Alignment) const;
	FBlock* AddDefaultBlock();
	FBlock* AddCustomBlock(SIZE_T UserSize, uint32 UserAlignment);
	FBlock* AllocateBlock(uint32 BlockSize, uint32 BlockAlignment);

	mutable FCriticalSection Lock;
	FMalloc* InnerMalloc = nullptr; // Pointer is constant during threading.
	FBlock* FirstHeader = nullptr; // Guarded by Lock
	FBlock* LastHeader = nullptr; // Guarded by Lock
	SIZE_T TotalAllocated = 0; // Guarded by Lock
	uint32 DefaultBlockSize = 0; // constant during threading
	ESPMode ThreadMode = ESPMode::ThreadSafe; // constant during threading
	std::atomic<int32> ReferenceCount{ 0 };

	friend FLinearBlockAllocatorThreadAccessor;
};

/**
 * Accessor for a LinearBlockAllocator when multithreading. Every thread using the Allocator must access it through
 * its own FLinearBlockAllocatorThreadAccessor.
 *
 * Intended use is to share an allocator amongst the task threads used within a single function in e.g. ParallelFor.
 *
 * The lifetime of FLinearBlockAllocatorThreadAccessor must be within the lifetime of the Allocator, an assertion
 * fires if FLinearBlockAllocatorThreadAccessor are still allocated when the Allocator is destructed. So the thread
 * that destroys the FLinearBlockAllocator must wait for all other threads using the Allocator to destroy their
 * FLinearBlockAllocatorThreadAccessor.
 */
class FLinearBlockAllocatorThreadAccessor : public FNoncopyable
{
public:
	CORE_API explicit FLinearBlockAllocatorThreadAccessor(FLinearBlockAllocator& InAllocator);
	CORE_API ~FLinearBlockAllocatorThreadAccessor();
	CORE_API FLinearBlockAllocatorThreadAccessor(FLinearBlockAllocatorThreadAccessor&& Other);

	// Copy constructor/assignment is disabled because each copy needs to get its own LastBlock from the allocator.
	// Move assignment is disabled because it would waste the remaining available memory allocated for *this.
	FLinearBlockAllocatorThreadAccessor& operator=(FLinearBlockAllocatorThreadAccessor&& Other) = delete;

	CORE_API void* Malloc(SIZE_T Size, uint32 Alignment = DEFAULT_ALIGNMENT);

private:
	FLinearBlockAllocator* Allocator = nullptr;
	FLinearBlockAllocator::FBlock* LastHeader = nullptr;
};

}

#if UE_ENABLE_LINEAR_VIRTUAL_ALLOCATOR

/**
 * A more efficient LinearAllocator than UE::FLinearBlockAllocator, but one tuned for global use for Unreal's persistent
 * core and engine allocations. This allocator should not be used by other systems.
 */
struct FLinearAllocator
{
	CORE_API FLinearAllocator(SIZE_T ReserveMemorySize);
	UE_FORCEINLINE_HINT ~FLinearAllocator()
	{
		VirtualMemory.FreeVirtual();
	}

	CORE_API void* Allocate(SIZE_T Size, uint32 Alignment = DEFAULT_ALIGNMENT);

	UE_FORCEINLINE_HINT SIZE_T GetAllocatedMemorySize() const
	{
		return Committed;
	}

protected:
	FCriticalSection Lock;
	FPlatformMemory::FPlatformVirtualMemoryBlock VirtualMemory;
	SIZE_T Reserved;
	SIZE_T Committed = 0;
	SIZE_T CurrentOffset = 0;

	bool CanFit(SIZE_T Size, uint32 Alignment) const;
	bool ContainsPointer(const void* Ptr) const;
};

#else

struct FLinearAllocator : public UE::FLinearBlockAllocator
{
	CORE_API FLinearAllocator(SIZE_T ReserveMemorySize);
	void* Allocate(SIZE_T Size, uint32 Alignment = DEFAULT_ALIGNMENT)
	{
		return Malloc(Size, Alignment);
	}
};

#endif	//~UE_ENABLE_LINEAR_VIRTUAL_ALLOCATOR

CORE_API FLinearAllocator& GetPersistentLinearAllocator();

struct FPersistentLinearAllocatorExtends
{
	uint64 Address = 0;
	uint64 Size = 0;
};

// Special case for the FPermanentObjectPoolExtents to reduce the amount of pointer dereferencing
extern CORE_API FPersistentLinearAllocatorExtends GPersistentLinearAllocatorExtends;