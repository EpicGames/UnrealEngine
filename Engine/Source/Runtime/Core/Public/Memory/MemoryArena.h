// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if !defined(UE_RESTRICT)
#	if defined _MSC_VER
#		define UE_RESTRICT __declspec(restrict)
#	else
#		define UE_RESTRICT
#	endif
#endif

#if !defined(UE_NOALIAS)
#	if defined _MSC_VER
#		define UE_NOALIAS __declspec(noalias)
#	else
#		define UE_NOALIAS
#	endif
#endif

#define UE_WITH_HEAPARENA	0
#define UE_WITH_ARENAMAP	0

class FHeapArena;
class FMemoryArena;

/** Tagged Arena Pointer

	Stores an arena ID alongside a pointer. On 64-bit architectures the arena ID
	can be encoded in the otherwise unused MSB part of the pointer for zero
	memory overhead vs a regular pointer. For 32-bit architectures the arena ID
	is stored in a separate member.

	A null pointer can still have an arena ID, which is useful for cases where
	you have a container that starts out empty but still want to track which arena
	should be used for subsequent allocations.

  */
class FArenaPointer
{
public:
	FArenaPointer() = default;

	enum { NoTag = 0 };

private:
	static constexpr int	ArenaShift	= 48;
	static constexpr uint64 ArenaMask	= 0xffff000000000000;
	static constexpr uint64 PointerMask = 0x0000ffffFFFFffff;

	void*	TaggedPointer = nullptr;

public:
	UE_FORCEINLINE_HINT const uint16	GetArenaIndex() const	{ return uint16(UPTRINT(TaggedPointer) >> ArenaShift); }
	UE_FORCEINLINE_HINT void*			GetPointer()			{ return reinterpret_cast<void*>(UPTRINT(TaggedPointer) & PointerMask); }
	UE_FORCEINLINE_HINT const void*		GetPointer() const		{ return reinterpret_cast<const void*>(UPTRINT(TaggedPointer) & PointerMask); }
	UE_FORCEINLINE_HINT void			SetPointerAndArena(void* InPtr, uint16 InArenaTag) { TaggedPointer = reinterpret_cast<void*>(UPTRINT(InPtr) | (UPTRINT(InArenaTag) << ArenaShift)); }

	inline FArenaPointer(void* Ptr, uint16 ArenaIndex) : TaggedPointer(reinterpret_cast<void*>(UPTRINT(Ptr) | (UPTRINT(ArenaIndex) << ArenaShift))) {};

	inline operator bool() const	{ return !!GetPointer(); }

	void Free() const;

	CORE_API FMemoryArena& Arena() const;
};

template<typename T>
class TArenaPointer : public FArenaPointer
{
public:
	TArenaPointer() = default;
	TArenaPointer(T* Ptr, uint16 ArenaIndex) : FArenaPointer(reinterpret_cast<T*>(Ptr), ArenaIndex) {}

	inline TArenaPointer& operator=(T* Rhs) { SetPointerAndArena(Rhs, NoTag); return *this; }
	
	inline operator T* ()					{ return reinterpret_cast<T*>(GetPointer()); }
	inline operator const T* () const		{ return reinterpret_cast<const T*>(GetPointer()); }
	inline T* operator -> ()				{ return reinterpret_cast<T*>(GetPointer()); }
	inline const T* operator -> () const	{ return reinterpret_cast<const T*>(GetPointer()); }
};

/** Memory arena interface
  */
class FMemoryArena
{
public:
	CORE_API			FMemoryArena();
	CORE_API virtual	~FMemoryArena();

	FMemoryArena(const FMemoryArena&) = delete;
	FMemoryArena& operator=(const FMemoryArena&) = delete;

	CORE_API UE_RESTRICT UE_NOALIAS void*	Alloc(SIZE_T Size, SIZE_T Alignment);
	CORE_API UE_NOALIAS void				Free(const void* MemoryBlock);

	CORE_API SIZE_T				GetBlockSize(const void* MemoryBlock) const;
	CORE_API const TCHAR*		GetDebugName() const;

private:
	virtual void*				InternalAlloc(SIZE_T Size, SIZE_T Alignment) = 0;
	CORE_API virtual void		InternalFree(const void* MemoryBlock, SIZE_T MemoryBlockSize);
	virtual SIZE_T				InternalGetBlockSize(const void* MemoryBlock) const = 0;

	CORE_API virtual const TCHAR* InternalGetDebugName() const;

	enum { FlagNoFree = 1 << 0 };

	uint16			ArenaFlags	= 0;

public:
	uint16			ArenaId		= ~uint16(0);
};

inline void FArenaPointer::Free() const 
{ 
	return Arena().Free(GetPointer());
}

// These are somewhat temporary, to support (experimental) arena-based container allocators

CORE_API FArenaPointer ArenaRealloc(FArenaPointer InPtr, SIZE_T OldSize, SIZE_T NewSize, SIZE_T Alignment);
CORE_API FArenaPointer ArenaRealloc(FMemoryArena* Arena, void* InPtr, SIZE_T OldSize, SIZE_T NewSize, SIZE_T Alignment);

/** Heap arena

	Manages a dedicated area of memory, and allows user to allocate blocks from
	it
  */

#if UE_WITH_HEAPARENA
class FHeapArena : public FMemoryArena
{
public:
	CORE_API			FHeapArena();
	CORE_API			~FHeapArena();

private:
	CORE_API virtual void*			InternalAlloc(SIZE_T Size, SIZE_T Alignment) override;
	CORE_API virtual void			InternalFree(const void* MemoryBlock, SIZE_T MemoryBlockSize) override;
	CORE_API virtual SIZE_T			InternalGetBlockSize(const void* MemoryBlock) const override;
	CORE_API virtual const TCHAR*	InternalGetDebugName() const override;

	void* HeapHandle = nullptr;
};
#endif

/** Default heap allocator

	All allocations are passed through to UE's main heap allocation functions

  */
class FMallocArena final : public FMemoryArena
{
public:
	CORE_API			FMallocArena();
	CORE_API			~FMallocArena();

private:
	CORE_API virtual void*			InternalAlloc(SIZE_T Size, SIZE_T Alignment) override;
	CORE_API virtual void			InternalFree(const void* MemoryBlock, SIZE_T MemoryBlockSize) override;
	CORE_API virtual SIZE_T			InternalGetBlockSize(const void* MemoryBlock) const override;
	CORE_API virtual const TCHAR*	InternalGetDebugName() const override;
};

/** CRT heap allocator

	All allocations are passed through to CRT memory allocation functions

  */
class FAnsiArena final : public FMemoryArena
{
public:
	CORE_API			FAnsiArena();
	CORE_API			~FAnsiArena();

private:
	CORE_API virtual void*			InternalAlloc(SIZE_T Size, SIZE_T Alignment) override;
	CORE_API virtual void			InternalFree(const void* MemoryBlock, SIZE_T MemoryBlockSize) override;
	CORE_API virtual SIZE_T			InternalGetBlockSize(const void* MemoryBlock) const override;
	CORE_API virtual const TCHAR*	InternalGetDebugName() const override;
};

/** Memory arena map

	Provides an efficient mechanism for mapping pointers to arenas
  */

#if UE_WITH_ARENAMAP
class FArenaMap
{
public:
	CORE_API static void Initialize();

	// Reset the state of the arena map - this is here only for testing and benchmarking purposes
	// and would never normally be called during normal execution
	CORE_API static void Reset();

	// Associate an arena with a virtual address range
	CORE_API static void SetRangeToArena(const void* VaBase, SIZE_T VaSize, FMemoryArena* ArenaPtr);

	// Mark range as unallocated
	CORE_API static void ClearRange(const void* VaBase, SIZE_T VaSize);

	// Map a virtual address to a memory arena
	CORE_API static FMemoryArena* MapPtrToArena(const void* VaBase);
};
#endif
