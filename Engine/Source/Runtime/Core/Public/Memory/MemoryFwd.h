// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/Void.h"

// MemoryView

template <UE::CVoid DataType>
class TMemoryView;

/** A non-owning view of a contiguous region of memory. */
using FMemoryView = TMemoryView<const void>;

/** A non-owning mutable view of a contiguous region of memory. */
using FMutableMemoryView = TMemoryView<void>;

// SharedBuffer
class FBufferOwner;
class FUniqueBuffer;
class FSharedBuffer;
class FWeakSharedBuffer;

// CompositeBuffer
class FCompositeBuffer;
