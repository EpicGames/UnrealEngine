// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/ContiguousRange.h"
#include "CoreTypes.h"
#include "HAL/PlatformString.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Memory/MemoryFwd.h" // IWYU pragma: export
#include "Templates/AndOrNot.h"
#include "Templates/EnableIf.h"
#include "Templates/Identity.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/UnrealTemplate.h"
#include "Traits/IsContiguousContainer.h"

#include <initializer_list>
#include <type_traits>

/**
 * A non-owning view of a contiguous region of memory.
 *
 * Prefer to use the aliases FMemoryView or FMutableMemoryView over this type.
 *
 * Functions that modify a view clamp sizes and offsets to always return a sub-view of the input.
 */
template <UE::CVoid DataType>
class TMemoryView
{
	using ByteType = std::conditional_t<std::is_const_v<DataType>, const uint8, uint8>;

public:
	/** Construct an empty view. */
	constexpr TMemoryView() = default;

	/** Construct a view of by copying a view with compatible const/volatile qualifiers. */
	template <
		typename OtherDataType
		UE_REQUIRES(std::is_convertible_v<OtherDataType*, DataType*>)
	>
	constexpr inline TMemoryView(const TMemoryView<OtherDataType>& InView)
		: Data(InView.Data)
		, Size(InView.Size)
	{
	}

	/** Construct a view of InSize bytes starting at InData. */
	constexpr inline TMemoryView(DataType* InData, uint64 InSize)
		: Data(InData)
		, Size(InSize)
	{
	}

	/** Construct a view starting at InData and ending at InDataEnd. */
	template <
		typename DataEndType
		UE_REQUIRES(std::is_convertible_v<DataEndType*, DataType*>)
	>
	inline TMemoryView(DataType* InData, DataEndType* InDataEnd)
		: Data(InData)
		, Size(static_cast<uint64>(static_cast<ByteType*>(ImplicitConv<DataType*>(InDataEnd)) - static_cast<ByteType*>(InData)))
	{
	}

	/** Construct a view from a contiguous range of any type with compatible const/volatile qualifiers. */
	template <UE::CContiguousRange RangeType>
		requires std::is_convertible_v<decltype(GetDataHelper(DeclVal<RangeType>())), DataType*>
	constexpr inline TMemoryView(RangeType& Range)
		: TMemoryView(GetDataHelper(Range), (uint64)GetNum(Range) * sizeof(*GetDataHelper(Range)))
	{
	}

	/** Returns a pointer to the start of the view. */
	[[nodiscard]] constexpr inline DataType* GetData() const { return Data; }

	/** Returns a pointer to the end of the view. */
	[[nodiscard]] inline DataType* GetDataEnd() const { return GetDataAtOffsetNoCheck(Size); }

	/** Returns the number of bytes in the view. */
	[[nodiscard]] constexpr inline uint64 GetSize() const { return Size; }

	/** Returns whether the view has a size of 0 regardless of its data pointer. */
	[[nodiscard]] constexpr inline bool IsEmpty() const { return Size == 0; }

	/** Resets to an empty view. */
	constexpr inline void Reset() { *this = TMemoryView(); }

	/** Returns the left-most part of the view by taking the given number of bytes from the left. */
	[[nodiscard]] constexpr inline TMemoryView Left(uint64 InSize) const
	{
		TMemoryView View(*this);
		View.LeftInline(InSize);
		return View;
	}

	/** Returns the left-most part of the view by chopping the given number of bytes from the right. */
	[[nodiscard]] constexpr inline TMemoryView LeftChop(uint64 InSize) const
	{
		TMemoryView View(*this);
		View.LeftChopInline(InSize);
		return View;
	}

	/** Returns the right-most part of the view by taking the given number of bytes from the right. */
	[[nodiscard]] inline TMemoryView Right(uint64 InSize) const
	{
		TMemoryView View(*this);
		View.RightInline(InSize);
		return View;
	}

	/** Returns the right-most part of the view by chopping the given number of bytes from the left. */
	[[nodiscard]] inline TMemoryView RightChop(uint64 InSize) const
	{
		TMemoryView View(*this);
		View.RightChopInline(InSize);
		return View;
	}

	/** Returns the middle part of the view by taking up to the given number of bytes from the given position. */
	[[nodiscard]] inline TMemoryView Mid(uint64 InOffset, uint64 InSize = TNumericLimits<uint64>::Max()) const
	{
		TMemoryView View(*this);
		View.MidInline(InOffset, InSize);
		return View;
	}

	/** Modifies the view to be the given number of bytes from the left. */
	constexpr inline void LeftInline(uint64 InSize)
	{
		Size = FMath::Min(Size, InSize);
	}

	/** Modifies the view by chopping the given number of bytes from the right. */
	constexpr inline void LeftChopInline(uint64 InSize)
	{
		Size -= FMath::Min(Size, InSize);
	}

	/** Modifies the view to be the given number of bytes from the right. */
	inline void RightInline(uint64 InSize)
	{
		const uint64 OldSize = Size;
		const uint64 NewSize = FMath::Min(OldSize, InSize);
		Data = GetDataAtOffsetNoCheck(OldSize - NewSize);
		Size = NewSize;
	}

	/** Modifies the view by chopping the given number of bytes from the left. */
	inline void RightChopInline(uint64 InSize)
	{
		const uint64 Offset = FMath::Min(Size, InSize);
		Data = GetDataAtOffsetNoCheck(Offset);
		Size -= Offset;
	}

	/** Modifies the view to be the middle part by taking up to the given number of bytes from the given offset. */
	inline void MidInline(uint64 InOffset, uint64 InSize = TNumericLimits<uint64>::Max())
	{
		RightChopInline(InOffset);
		LeftInline(InSize);
	}

	/** Returns whether this view fully contains the other view. */
	template <typename OtherDataType>
	[[nodiscard]] inline bool Contains(const TMemoryView<OtherDataType>& InView) const
	{
		return Data <= InView.Data && GetDataAtOffsetNoCheck(Size) >= InView.GetDataAtOffsetNoCheck(InView.Size);
	}

	/** Returns whether this view intersects the other view. */
	template <typename OtherDataType>
	[[nodiscard]] inline bool Intersects(const TMemoryView<OtherDataType>& InView) const
	{
		return Data < InView.GetDataAtOffsetNoCheck(InView.Size) && InView.Data < GetDataAtOffsetNoCheck(Size);
	}

	/** Returns whether the bytes of this view are equal or less/greater than the bytes of the other view. */
	template <typename OtherDataType>
	[[nodiscard]] inline int32 CompareBytes(const TMemoryView<OtherDataType>& InView) const
	{
		const int32 Compare = Data == InView.Data ? 0 : FMemory::Memcmp(Data, InView.Data, FMath::Min(Size, InView.Size));
		return Compare || Size == InView.Size ? Compare : Size < InView.Size ? -1 : 1;
	}

	/** Returns whether the bytes of this views are equal to the bytes of the other view. */
	template <typename OtherDataType>
	[[nodiscard]] inline bool EqualBytes(const TMemoryView<OtherDataType>& InView) const
	{
		return Size == InView.Size && (Data == InView.Data || FMemory::Memcmp(Data, InView.Data, Size) == 0);
	}

	/** Returns whether the data pointers and sizes of this view and the other view are equal. */
	template <typename OtherDataType>
	[[nodiscard]] constexpr inline bool Equals(const TMemoryView<OtherDataType>& InView) const
	{
		return Size == InView.Size && (Size == 0 || Data == InView.Data);
	}

	/** Returns whether the data pointers and sizes of this view and the other view are equal. */
	template <typename OtherDataType>
	[[nodiscard]] constexpr inline bool operator==(const TMemoryView<OtherDataType>& InView) const
	{
		return Equals(InView);
	}

	/** Returns whether the data pointers and sizes of this view and the other view are not equal. */
	template <typename OtherDataType>
	[[nodiscard]] constexpr inline bool operator!=(const TMemoryView<OtherDataType>& InView) const
	{
		return !Equals(InView);
	}

	/** Advances the start of the view by an offset, which is clamped to stay within the view. */
	constexpr inline TMemoryView& operator+=(uint64 InOffset)
	{
		RightChopInline(InOffset);
		return *this;
	}

	/** Copies bytes from the input view into this view, and returns the remainder of this view. */
	inline TMemoryView CopyFrom(FMemoryView InView) const
	{
		checkf(InView.Size <= Size, TEXT("Failed to copy from a view of %" UINT64_FMT " bytes "
			"to a view of %" UINT64_FMT " bytes."), InView.Size, Size);
		if (InView.Size)
		{
			FMemory::Memcpy(Data, InView.Data, InView.Size);
		}
		return RightChop(InView.Size);
	}

private:
	/** Returns the data pointer advanced by an offset in bytes. */
	[[nodiscard]] inline DataType* GetDataAtOffsetNoCheck(uint64 InOffset) const
	{
		return reinterpret_cast<ByteType*>(Data) + InOffset;
	}

	template <UE::CVoid OtherDataType>
	friend class TMemoryView;

private:
	DataType* Data = nullptr;
	uint64 Size = 0;
};

/** Advances the start of the view by an offset, which is clamped to stay within the view. */
template <typename DataType>
[[nodiscard]] constexpr inline TMemoryView<DataType> operator+(const TMemoryView<DataType>& View, uint64 Offset)
{
	return TMemoryView<DataType>(View) += Offset;
}

/** Advances the start of the view by an offset, which is clamped to stay within the view. */
template <typename DataType>
[[nodiscard]] constexpr inline TMemoryView<DataType> operator+(uint64 Offset, const TMemoryView<DataType>& View)
{
	return TMemoryView<DataType>(View) += Offset;
}

/** Make a non-owning mutable view of Size bytes starting at Data. */
[[nodiscard]] constexpr inline TMemoryView<void> MakeMemoryView(void* Data, uint64 Size)
{
	return TMemoryView<void>(Data, Size);
}

/** Make a non-owning const view of Size bytes starting at Data. */
[[nodiscard]] constexpr inline TMemoryView<const void> MakeMemoryView(const void* Data, uint64 Size)
{
	return TMemoryView<const void>(Data, Size);
}

/** Make a non-owning view starting at Data and ending at DataEnd. */
template <typename DataType, typename DataEndType>
[[nodiscard]] inline auto MakeMemoryView(DataType* Data, DataEndType* DataEnd)
{
	// This function is templated on pointer type to prevent MakeMemoryView(Ptr, 0) being deduced as a null end pointer

	using VoidType = std::conditional_t<std::is_const_v<DataType> || std::is_const_v<DataEndType>, const void, void>;

	return TMemoryView<VoidType>(Data, DataEnd);
}

/**
 * Make a non-owning view of the memory of the initializer list.
 *
 * This overload is only available when the element type does not need to be deduced.
 */
template <typename T>
[[nodiscard]] constexpr inline TMemoryView<const void> MakeMemoryView(std::initializer_list<typename TIdentity<T>::Type> List)
{
	return TMemoryView<const void>(GetData(List), GetNum(List) * sizeof(T));
}

/** Make a non-owning view of the memory of the contiguous container. */
template <
	typename ContainerType
	UE_REQUIRES(TIsContiguousContainer<ContainerType>::Value)
>
[[nodiscard]] constexpr inline auto MakeMemoryView(ContainerType&& Container)
{
	using ElementType = typename TRemovePointer<decltype(GetData(DeclVal<ContainerType>()))>::Type;
	constexpr bool bIsConst = std::is_const_v<ElementType>;
	using DataType = std::conditional_t<bIsConst, const void, void>;
	return TMemoryView<DataType>(GetData(Container), GetNum(Container) * sizeof(ElementType));
}

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Templates/IsConst.h"
#endif
