// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "CoreTypes.h"
#include "Memory/MemoryFwd.h"
#include "Memory/MemoryView.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Invoke.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"

#include <atomic>
#include <type_traits>

template <typename T> struct TIsWeakPointerType;
template <typename T> struct TIsZeroConstructType;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::SharedBuffer::Private { struct FSharedOps; }
namespace UE::SharedBuffer::Private { struct FWeakOps; }

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * A reference-counted owner for a buffer, which is a raw pointer and size.
 *
 * A buffer owner may own its memory or provide a view into memory owned externally. When used as
 * a non-owning view, the viewed memory must be guaranteed to outlive the buffer owner. When this
 * lifetime guarantee cannot be satisfied, MakeOwned may be called on the reference to the buffer
 * to clone into a new buffer owner that owns the memory.
 *
 * A buffer owner must be referenced and accessed through one of its three reference types:
 * FUniqueBuffer, FSharedBuffer, or FWeakSharedBuffer.
 *
 * FUniqueBuffer and FSharedBuffer offer static functions to create a buffer using private buffer
 * owner types that are ideal for most use cases. The TakeOwnership function allows creation of a
 * buffer with a custom delete function. Advanced use cases require deriving from FBufferOwner to
 * enable storage of arbitrary members alongside the data and size, and to enable materialization
 * of the buffer to be deferred.
 *
 * A derived type must call SetIsOwned from its constructor if they own (or will own) the buffer.
 * A derived type must call SetIsImmutable from its constructor if they want the buffer to become
 * permanently immutable after it is stored in FSharedBuffer, forcing MoveToUnique() to clone it.
 * A derived type must call SetIsMaterialized from its constructor, unless it implements deferred
 * materialization by overriding MaterializeBuffer.
 */
class FBufferOwner
{
private:
	enum class EBufferOwnerFlags : uint8;
	FRIEND_ENUM_CLASS_FLAGS(EBufferOwnerFlags);

protected:
	FBufferOwner() = default;

	FBufferOwner(const FBufferOwner&) = delete;
	FBufferOwner& operator=(const FBufferOwner&) = delete;

	inline FBufferOwner(void* InData, uint64 InSize);
	virtual ~FBufferOwner();

	/**
	 * Materialize the buffer by making it ready to be accessed.
	 *
	 * This will be called before any access to the data or size, unless SetIsMaterialized is called
	 * by the constructor. Accesses from multiple threads will cause multiple calls to this function
	 * until at least one has finished.
	 */
	virtual void MaterializeBuffer();

	/**
	 * Free the buffer and any associated resources.
	 *
	 * This is called when the last shared reference is released. The destructor will be called when
	 * the last weak reference is released. A buffer owner will always call this function before the
	 * calling the destructor, unless an exception was thrown by the constructor.
	 */
	virtual void FreeBuffer() = 0;

	inline void* GetData();
	inline uint64 GetSize();
	inline void SetBuffer(void* InData, uint64 InSize);

	inline bool IsOwned() const;
	inline void SetIsOwned();

	inline bool IsImmutable() const;
	inline void SetIsImmutable();

	inline void Materialize();
	inline bool IsMaterialized() const;
	inline void SetIsMaterialized();

	inline uint32 GetTotalRefCount() const;

private:
	static constexpr uint32 RefCountMask = 0x3fffffff;

	static constexpr inline uint32 GetTotalRefCount(uint64 RefCountsAndFlags);
	static constexpr inline uint32 GetSharedRefCount(uint64 RefCountsAndFlags) { return uint32(RefCountsAndFlags >> 0) & RefCountMask; }
	static constexpr inline uint64 SetSharedRefCount(uint32 RefCount) { return uint64(RefCount) << 0; }
	static constexpr inline uint32 GetWeakRefCount(uint64 RefCountsAndFlags) { return uint32(RefCountsAndFlags >> 30) & RefCountMask; }
	static constexpr inline uint64 SetWeakRefCount(uint32 RefCount) { return uint64(RefCount) << 30; }
	static constexpr inline EBufferOwnerFlags GetFlags(uint64 RefCountsAndFlags) { return EBufferOwnerFlags(RefCountsAndFlags >> 60); }
	static constexpr inline uint64 SetFlags(EBufferOwnerFlags Flags) { return uint64(Flags) << 60; }

	/** Returns whether this has one total reference, is owned, and is not immutable. */
	inline bool IsUniqueOwnedMutable() const;

	inline void AddSharedReference();
	inline void ReleaseSharedReference();
	inline bool TryAddSharedReference();
	inline void AddWeakReference();
	inline void ReleaseWeakReference();

	friend class FUniqueBuffer;
	friend class FSharedBuffer;
	friend class FWeakSharedBuffer;
	friend struct UE::SharedBuffer::Private::FSharedOps;
	friend struct UE::SharedBuffer::Private::FWeakOps;

private:
	void* Data = nullptr;
	uint64 Size = 0;
	std::atomic<uint64> ReferenceCountsAndFlags{0};
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::SharedBuffer::Private
{

struct FSharedOps final
{
	static inline bool HasRef(FBufferOwner& Owner) { return Owner.GetTotalRefCount() > 0; }
	static inline bool TryAddRef(FBufferOwner& Owner) { return Owner.TryAddSharedReference(); }
	static inline void AddRef(FBufferOwner& Owner) { Owner.AddSharedReference(); }
	static inline void Release(FBufferOwner* Owner) { if (Owner) { Owner->ReleaseSharedReference(); } }
};

struct FWeakOps final
{
	static inline bool HasRef(FBufferOwner& Owner) { return Owner.GetTotalRefCount() > 0; }
	static inline bool TryAddRef(FBufferOwner& Owner) { AddRef(Owner); return true; }
	static inline void AddRef(FBufferOwner& Owner) { Owner.AddWeakReference(); }
	static inline void Release(FBufferOwner* Owner) { if (Owner) { Owner->ReleaseWeakReference(); } }
};

template <typename FOps>
class TBufferOwnerPtr final
{
	static constexpr bool bIsWeak = std::is_same<FOps, FWeakOps>::value;

	template <typename FOtherOps>
	friend class TBufferOwnerPtr;

	template <typename FOtherOps>
	static inline FBufferOwner* CopyFrom(const TBufferOwnerPtr<FOtherOps>& Ptr);

	template <typename FOtherOps>
	static inline FBufferOwner* MoveFrom(TBufferOwnerPtr<FOtherOps>&& Ptr);

public:
	inline TBufferOwnerPtr() = default;
	inline explicit TBufferOwnerPtr(FBufferOwner* const InOwner);

	inline TBufferOwnerPtr(const TBufferOwnerPtr& Ptr);
	inline TBufferOwnerPtr(TBufferOwnerPtr&& Ptr);

	template <typename FOtherOps>
	inline explicit TBufferOwnerPtr(const TBufferOwnerPtr<FOtherOps>& Ptr);
	template <typename FOtherOps>
	inline explicit TBufferOwnerPtr(TBufferOwnerPtr<FOtherOps>&& Ptr);

	inline ~TBufferOwnerPtr();

	inline TBufferOwnerPtr& operator=(const TBufferOwnerPtr& Ptr);
	inline TBufferOwnerPtr& operator=(TBufferOwnerPtr&& Ptr);

	template <typename FOtherOps>
	inline TBufferOwnerPtr& operator=(const TBufferOwnerPtr<FOtherOps>& Ptr);
	template <typename FOtherOps>
	inline TBufferOwnerPtr& operator=(TBufferOwnerPtr<FOtherOps>&& Ptr);

	template <typename FOtherOps>
	inline bool operator==(const TBufferOwnerPtr<FOtherOps>& Ptr) const;
	template <typename FOtherOps>
	inline bool operator!=(const TBufferOwnerPtr<FOtherOps>& Ptr) const;

	inline FBufferOwner* Get() const { return Owner; }
	inline FBufferOwner* operator->() const { return Get(); }
	inline explicit operator bool() const { return !IsNull(); }
	inline bool IsNull() const { return Owner == nullptr; }

	inline void Reset();

private:
	FBufferOwner* Owner = nullptr;
};

} // UE::SharedBuffer::Private

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * A reference to a single-ownership mutable buffer.
 *
 * Ownership can be transferred by moving to FUniqueBuffer or it can be converted to an immutable
 * shared buffer with MoveToShared().
 *
 * @see FBufferOwner
 */
class FUniqueBuffer
{
public:
	/** Make an uninitialized owned buffer of the specified size. */
	[[nodiscard]] CORE_API static FUniqueBuffer Alloc(uint64 Size);
	/** Make a zeroed owned buffer of the specified size. */
	[[nodiscard]] CORE_API static FUniqueBuffer AllocZeroed(uint64 Size);

	/** Make an owned clone of the input. */
	[[nodiscard]] CORE_API static FUniqueBuffer Clone(FMemoryView View);
	[[nodiscard]] CORE_API static FUniqueBuffer Clone(const void* Data, uint64 Size);

	/** Make a non-owned view of the input. */
	[[nodiscard]] CORE_API static FUniqueBuffer MakeView(FMutableMemoryView View);
	[[nodiscard]] CORE_API static FUniqueBuffer MakeView(void* Data, uint64 Size);

	/**
	 * Make an owned buffer by taking ownership of the input.
	 *
	 * @param DeleteFunction Called with Data to free memory when the last shared reference is released.
	 */
	template <typename DeleteFunctionType,
		decltype(Invoke(std::declval<DeleteFunctionType>(), std::declval<void*>()))* = nullptr>
	[[nodiscard]] static inline FUniqueBuffer TakeOwnership(void* Data, uint64 Size, DeleteFunctionType&& DeleteFunction);

	/**
	 * Make an owned buffer by taking ownership of the input.
	 *
	 * @param DeleteFunction Called with (Data, Size) to free memory when the last shared reference is released.
	 */
	template <typename DeleteFunctionType,
		decltype(Invoke(std::declval<DeleteFunctionType>(), std::declval<void*>(), std::declval<uint64>()))* = nullptr>
	[[nodiscard]] static inline FUniqueBuffer TakeOwnership(void* Data, uint64 Size, DeleteFunctionType&& DeleteFunction);

	/** Construct a null unique buffer. */
	FUniqueBuffer() = default;

	/** Construct a unique buffer from a new unreferenced buffer owner. */
	CORE_API explicit FUniqueBuffer(FBufferOwner* Owner);

	FUniqueBuffer(FUniqueBuffer&&) = default;
	FUniqueBuffer& operator=(FUniqueBuffer&&) = default;

	FUniqueBuffer(const FUniqueBuffer&) = delete;
	FUniqueBuffer& operator=(const FUniqueBuffer&) = delete;

	/** Reset this to null. */
	CORE_API void Reset();

	/** Returns a pointer to the start of the buffer. */
	[[nodiscard]] inline void* GetData() { return Owner ? Owner->GetData() : nullptr; }
	[[nodiscard]] inline const void* GetData() const { return Owner ? Owner->GetData() : nullptr; }

	/** Returns the size of the buffer in bytes. */
	[[nodiscard]] inline uint64 GetSize() const { return Owner ? Owner->GetSize() : 0; }

	/** Returns a view of the buffer. */
	[[nodiscard]] inline FMutableMemoryView GetView() { return FMutableMemoryView(GetData(), GetSize()); }
	[[nodiscard]] inline FMemoryView GetView() const { return FMemoryView(GetData(), GetSize()); }
	[[nodiscard]] inline operator FMutableMemoryView() { return GetView(); }
	[[nodiscard]] inline operator FMemoryView() const { return GetView(); }

	/** Returns true if this points to a buffer owner. */
	[[nodiscard]] inline explicit operator bool() const { return !IsNull(); }

	/**
	 * Returns true if this does not point to a buffer owner.
	 *
	 * A null buffer is always owned, materialized, and empty.
	 */
	[[nodiscard]] inline bool IsNull() const { return Owner.IsNull(); }

	/** Returns true if this keeps the referenced buffer alive. */
	[[nodiscard]] inline bool IsOwned() const { return !Owner || Owner->IsOwned(); }

	/** Returns a buffer that is owned, by cloning if not owned. */
	[[nodiscard]] CORE_API FUniqueBuffer MakeOwned() &&;

	/** Returns true if the referenced buffer has been materialized. */
	[[nodiscard]] inline bool IsMaterialized() const { return !Owner || Owner->IsMaterialized(); }

	/**
	 * Materialize the buffer by making its data and size available.
	 *
	 * The buffer is automatically materialized by GetData, GetSize, GetView.
	 */
	CORE_API void Materialize() const;

	/**
	 * Convert this to an immutable shared buffer, leaving this null.
	 *
	 * Steals the buffer owner from the unique buffer.
	 */
	[[nodiscard]] CORE_API FSharedBuffer MoveToShared();

	friend class FSharedBuffer;

private:
	using FOwnerPtrType = UE::SharedBuffer::Private::TBufferOwnerPtr<UE::SharedBuffer::Private::FSharedOps>;

	explicit FUniqueBuffer(FOwnerPtrType&& SharedOwner);

	inline friend const FOwnerPtrType& ToPrivateOwnerPtr(const FUniqueBuffer& Buffer) { return Buffer.Owner; }
	inline friend FOwnerPtrType ToPrivateOwnerPtr(FUniqueBuffer&& Buffer) { return MoveTemp(Buffer.Owner); }

	FOwnerPtrType Owner;

public:
	friend inline uint32 GetTypeHash(const FUniqueBuffer& Buffer) { return PointerHash(ToPrivateOwnerPtr(Buffer).Get()); }

	inline bool operator==(const FUniqueBuffer& BufferB) const { return ToPrivateOwnerPtr(*this) == ToPrivateOwnerPtr(BufferB); }
#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	inline bool operator!=(const FUniqueBuffer& BufferB) const { return ToPrivateOwnerPtr(*this) != ToPrivateOwnerPtr(BufferB); }
#endif
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * A reference to a shared-ownership immutable buffer.
 *
 * @see FBufferOwner
 */
class FSharedBuffer
{
public:
	/** Make an owned clone of the input. */
	[[nodiscard]] CORE_API static FSharedBuffer Clone(FMemoryView View);
	[[nodiscard]] CORE_API static FSharedBuffer Clone(const void* Data, uint64 Size);

	/** Make a non-owned view of the input. */
	[[nodiscard]] CORE_API static FSharedBuffer MakeView(FMemoryView View);
	[[nodiscard]] CORE_API static FSharedBuffer MakeView(const void* Data, uint64 Size);

	/** Make a view of the input within its outer buffer. Ownership matches OuterBuffer. */
	[[nodiscard]] CORE_API static FSharedBuffer MakeView(FMemoryView View, FSharedBuffer&& OuterBuffer);
	[[nodiscard]] CORE_API static FSharedBuffer MakeView(FMemoryView View, const FSharedBuffer& OuterBuffer);
	[[nodiscard]] CORE_API static FSharedBuffer MakeView(const void* Data, uint64 Size, FSharedBuffer&& OuterBuffer);
	[[nodiscard]] CORE_API static FSharedBuffer MakeView(const void* Data, uint64 Size, const FSharedBuffer& OuterBuffer);

	/**
	 * Make an owned buffer by taking ownership of the input.
	 *
	 * @param DeleteFunction Called with Data to free memory when the last shared reference is released.
	 */
	template <typename DeleteFunctionType,
		decltype(Invoke(std::declval<DeleteFunctionType>(), std::declval<void*>()))* = nullptr>
	[[nodiscard]] static inline FSharedBuffer TakeOwnership(const void* Data, uint64 Size, DeleteFunctionType&& DeleteFunction);

	/**
	 * Make an owned buffer by taking ownership of the input.
	 *
	 * @param DeleteFunction Called with (Data, Size) to free memory when the last shared reference is released.
	 */
	template <typename DeleteFunctionType,
		decltype(Invoke(std::declval<DeleteFunctionType>(), std::declval<void*>(), std::declval<uint64>()))* = nullptr>
	[[nodiscard]] static inline FSharedBuffer TakeOwnership(const void* Data, uint64 Size, DeleteFunctionType&& DeleteFunction);

	/** Construct a null shared buffer. */
	FSharedBuffer() = default;

	/** Construct a shared buffer from a new unreferenced buffer owner. */
	CORE_API explicit FSharedBuffer(FBufferOwner* Owner);

	/** Reset this to null. */
	CORE_API void Reset();

	/** Returns a pointer to the start of the buffer. */
	[[nodiscard]] inline const void* GetData() const { return Owner ? Owner->GetData() : nullptr; }

	/** Returns the size of the buffer in bytes. */
	[[nodiscard]] inline uint64 GetSize() const { return Owner ? Owner->GetSize() : 0; }

	/** Returns a view of the buffer. */
	[[nodiscard]] inline FMemoryView GetView() const { return FMemoryView(GetData(), GetSize()); }
	[[nodiscard]] inline operator FMemoryView() const { return GetView(); }

	/** Returns true if this points to a buffer owner. */
	[[nodiscard]] inline explicit operator bool() const { return !IsNull(); }

	/**
	 * Returns true if this does not point to a buffer owner.
	 *
	 * A null buffer is always owned, materialized, and empty.
	 */
	[[nodiscard]] inline bool IsNull() const { return Owner.IsNull(); }

	/** Returns true if this keeps the referenced buffer alive. */
	[[nodiscard]] inline bool IsOwned() const { return !Owner || Owner->IsOwned(); }

	/** Returns a buffer that is owned, by cloning if not owned. */
	[[nodiscard]] CORE_API FSharedBuffer MakeOwned() const &;
	[[nodiscard]] CORE_API FSharedBuffer MakeOwned() &&;

	/** Returns true if the referenced buffer has been materialized. */
	[[nodiscard]] inline bool IsMaterialized() const { return !Owner || Owner->IsMaterialized(); }

	/**
	 * Materialize the buffer by making its data and size available.
	 *
	 * The buffer is automatically materialized by GetData, GetSize, GetView.
	 */
	CORE_API void Materialize() const;

	/**
	 * Convert this to a unique buffer, leaving this null.
	 *
	 * Steals the buffer owner from the shared buffer if this is the last reference to it, otherwise
	 * clones the shared buffer to guarantee unique ownership. A non-owned buffer is always cloned.
	 */
	[[nodiscard]] CORE_API FUniqueBuffer MoveToUnique();

	friend class FUniqueBuffer;
	friend class FWeakSharedBuffer;

private:
	using FOwnerPtrType = UE::SharedBuffer::Private::TBufferOwnerPtr<UE::SharedBuffer::Private::FSharedOps>;
	using FWeakOwnerPtrType = UE::SharedBuffer::Private::TBufferOwnerPtr<UE::SharedBuffer::Private::FWeakOps>;

	explicit FSharedBuffer(FOwnerPtrType&& SharedOwner);
	explicit FSharedBuffer(const FWeakOwnerPtrType& WeakOwner);

	inline friend const FOwnerPtrType& ToPrivateOwnerPtr(const FSharedBuffer& Buffer) { return Buffer.Owner; }
	inline friend FOwnerPtrType ToPrivateOwnerPtr(FSharedBuffer&& Buffer) { return MoveTemp(Buffer.Owner); }

	FOwnerPtrType Owner;

public:
	friend inline uint32 GetTypeHash(const FSharedBuffer& Buffer) { return PointerHash(ToPrivateOwnerPtr(Buffer).Get()); }
	inline bool operator==(const FSharedBuffer& BufferB) const { return ToPrivateOwnerPtr(*this) == ToPrivateOwnerPtr(BufferB); }
	friend inline bool operator==(const FSharedBuffer& BufferA, const FUniqueBuffer& BufferB) { return ToPrivateOwnerPtr(BufferA) == ToPrivateOwnerPtr(BufferB); }

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	inline bool operator!=(const FSharedBuffer& BufferB) const { return ToPrivateOwnerPtr(*this) != ToPrivateOwnerPtr(BufferB); }
	friend inline bool operator!=(const FSharedBuffer& BufferA, const FUniqueBuffer& BufferB) { return ToPrivateOwnerPtr(BufferA) != ToPrivateOwnerPtr(BufferB); }
	friend inline bool operator==(const FUniqueBuffer& BufferA, const FSharedBuffer& BufferB) { return ToPrivateOwnerPtr(BufferA) == ToPrivateOwnerPtr(BufferB); }
	friend inline bool operator!=(const FUniqueBuffer& BufferA, const FSharedBuffer& BufferB) { return ToPrivateOwnerPtr(BufferA) != ToPrivateOwnerPtr(BufferB); }
#endif
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * A weak reference to a shared-ownership immutable buffer.
 *
 * @see FSharedBuffer
 */
class FWeakSharedBuffer
{
public:
	/** Construct a null weak shared buffer. */
	FWeakSharedBuffer() = default;

	/** Construct a weak shared buffer from a shared buffer. */
	CORE_API FWeakSharedBuffer(const FSharedBuffer& Buffer);

	/** Assign a weak shared buffer from a shared buffer. */
	CORE_API FWeakSharedBuffer& operator=(const FSharedBuffer& Buffer);

	/** Reset this to null. */
	CORE_API void Reset();

	/** Convert this to a shared buffer if it has any remaining shared references. */
	[[nodiscard]] CORE_API FSharedBuffer Pin() const;

private:
	using FWeakOwnerPtrType = UE::SharedBuffer::Private::TBufferOwnerPtr<UE::SharedBuffer::Private::FWeakOps>;

	inline friend const FWeakOwnerPtrType& ToPrivateOwnerPtr(const FWeakSharedBuffer& Buffer) { return Buffer.Owner; }

	FWeakOwnerPtrType Owner;

public:
	friend inline uint32 GetTypeHash(const FWeakSharedBuffer& Buffer) { return PointerHash(ToPrivateOwnerPtr(Buffer).Get()); }
	inline bool operator==(const FWeakSharedBuffer& BufferB) const { return ToPrivateOwnerPtr(*this) == ToPrivateOwnerPtr(BufferB); }
	friend inline bool operator==(const FWeakSharedBuffer& BufferA, const FUniqueBuffer& BufferB) { return ToPrivateOwnerPtr(BufferA) == ToPrivateOwnerPtr(BufferB); }
	friend inline bool operator==(const FWeakSharedBuffer& BufferA, const FSharedBuffer& BufferB) { return ToPrivateOwnerPtr(BufferA) == ToPrivateOwnerPtr(BufferB); }

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	friend inline bool operator==(const FUniqueBuffer& BufferA, const FWeakSharedBuffer& BufferB) { return ToPrivateOwnerPtr(BufferA) == ToPrivateOwnerPtr(BufferB); }
	friend inline bool operator==(const FSharedBuffer& BufferA, const FWeakSharedBuffer& BufferB) { return ToPrivateOwnerPtr(BufferA) == ToPrivateOwnerPtr(BufferB); }

	inline bool operator!=(const FWeakSharedBuffer& BufferB) const { return ToPrivateOwnerPtr(*this) != ToPrivateOwnerPtr(BufferB); }
	friend inline bool operator!=(const FWeakSharedBuffer& BufferA, const FUniqueBuffer& BufferB) { return ToPrivateOwnerPtr(BufferA) != ToPrivateOwnerPtr(BufferB); }
	friend inline bool operator!=(const FUniqueBuffer& BufferA, const FWeakSharedBuffer& BufferB) { return ToPrivateOwnerPtr(BufferA) != ToPrivateOwnerPtr(BufferB); }
	friend inline bool operator!=(const FWeakSharedBuffer& BufferA, const FSharedBuffer& BufferB) { return ToPrivateOwnerPtr(BufferA) != ToPrivateOwnerPtr(BufferB); }
	friend inline bool operator!=(const FSharedBuffer& BufferA, const FWeakSharedBuffer& BufferB) { return ToPrivateOwnerPtr(BufferA) != ToPrivateOwnerPtr(BufferB); }
#endif
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

template<> struct TIsZeroConstructType<FUniqueBuffer> { enum { Value = true }; };
template<> struct TIsZeroConstructType<FSharedBuffer> { enum { Value = true }; };
template<> struct TIsZeroConstructType<FWeakSharedBuffer> { enum { Value = true }; };

template<> struct TIsWeakPointerType<FWeakSharedBuffer> { enum { Value = true }; };

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::SharedBuffer::Private
{

template <typename DeleteFunctionType>
class TBufferOwnerDeleteFunction final : public FBufferOwner
{
public:
	explicit TBufferOwnerDeleteFunction(void* Data, uint64 Size, DeleteFunctionType&& InDeleteFunction)
		: FBufferOwner(Data, Size)
		, DeleteFunction(Forward<DeleteFunctionType>(InDeleteFunction))
	{
		SetIsMaterialized();
		SetIsOwned();
	}

	explicit TBufferOwnerDeleteFunction(const void* Data, uint64 Size, DeleteFunctionType&& InDeleteFunction)
		: TBufferOwnerDeleteFunction(const_cast<void*>(Data), Size, Forward<DeleteFunctionType>(InDeleteFunction))
	{
		SetIsImmutable();
	}

private:
	virtual void FreeBuffer() final
	{
		Invoke(DeleteFunction, GetData(), GetSize());
	}

	std::decay_t<DeleteFunctionType> DeleteFunction;
};

} // UE::SharedBuffer::Private

template <typename DeleteFunctionType,
	decltype(Invoke(std::declval<DeleteFunctionType>(), std::declval<void*>()))*>
inline FUniqueBuffer FUniqueBuffer::TakeOwnership(void* Data, uint64 Size, DeleteFunctionType&& DeleteFunction)
{
	return TakeOwnership(Data, Size, [Delete = Forward<DeleteFunctionType>(DeleteFunction)](void* InData, uint64 InSize)
	{
		Invoke(Delete, InData);
	});
}

template <typename DeleteFunctionType,
	decltype(Invoke(std::declval<DeleteFunctionType>(), std::declval<void*>(), std::declval<uint64>()))*>
inline FUniqueBuffer FUniqueBuffer::TakeOwnership(void* Data, uint64 Size, DeleteFunctionType&& DeleteFunction)
{
	using OwnerType = UE::SharedBuffer::Private::TBufferOwnerDeleteFunction<DeleteFunctionType>;
	return FUniqueBuffer(new OwnerType(Data, Size, Forward<DeleteFunctionType>(DeleteFunction)));
}

template <typename DeleteFunctionType,
	decltype(Invoke(std::declval<DeleteFunctionType>(), std::declval<void*>()))*>
inline FSharedBuffer FSharedBuffer::TakeOwnership(const void* Data, uint64 Size, DeleteFunctionType&& DeleteFunction)
{
	return TakeOwnership(Data, Size, [Delete = Forward<DeleteFunctionType>(DeleteFunction)](void* InData, uint64 InSize)
	{
		Invoke(Delete, InData);
	});
}

template <typename DeleteFunctionType,
	decltype(Invoke(std::declval<DeleteFunctionType>(), std::declval<void*>(), std::declval<uint64>()))*>
inline FSharedBuffer FSharedBuffer::TakeOwnership(const void* Data, uint64 Size, DeleteFunctionType&& DeleteFunction)
{
	using OwnerType = UE::SharedBuffer::Private::TBufferOwnerDeleteFunction<DeleteFunctionType>;
	return FSharedBuffer(new OwnerType(Data, Size, Forward<DeleteFunctionType>(DeleteFunction)));
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

enum class FBufferOwner::EBufferOwnerFlags : uint8
{
	None         = 0,
	Owned        = 1 << 0,
	Immutable    = 1 << 1,
	Materialized = 1 << 2,
};

ENUM_CLASS_FLAGS(FBufferOwner::EBufferOwnerFlags);

inline FBufferOwner::FBufferOwner(void* InData, uint64 InSize)
	: Data(InData)
	, Size(InSize)
{
}

inline FBufferOwner::~FBufferOwner()
{
	checkSlow(GetTotalRefCount() == 0);
}

inline void FBufferOwner::MaterializeBuffer()
{
	SetIsMaterialized();
}

inline void* FBufferOwner::GetData()
{
	Materialize();
	return Data;
}

inline uint64 FBufferOwner::GetSize()
{
	Materialize();
	return Size;
}

inline void FBufferOwner::SetBuffer(void* InData, uint64 InSize)
{
	Data = InData;
	Size = InSize;
}

inline bool FBufferOwner::IsOwned() const
{
	return EnumHasAnyFlags(GetFlags(ReferenceCountsAndFlags.load(std::memory_order_relaxed)), EBufferOwnerFlags::Owned);
}

inline void FBufferOwner::SetIsOwned()
{
	ReferenceCountsAndFlags.fetch_or(SetFlags(EBufferOwnerFlags::Owned), std::memory_order_relaxed);
}

inline bool FBufferOwner::IsImmutable() const
{
	return EnumHasAnyFlags(GetFlags(ReferenceCountsAndFlags.load(std::memory_order_relaxed)), EBufferOwnerFlags::Immutable);
}

inline void FBufferOwner::SetIsImmutable()
{
	ReferenceCountsAndFlags.fetch_or(SetFlags(EBufferOwnerFlags::Immutable), std::memory_order_relaxed);
}

inline void FBufferOwner::Materialize()
{
	if (!IsMaterialized())
	{
		MaterializeBuffer();
		checkSlow(IsMaterialized());
	}
}

inline bool FBufferOwner::IsMaterialized() const
{
	return EnumHasAnyFlags(GetFlags(ReferenceCountsAndFlags.load(std::memory_order_acquire)), EBufferOwnerFlags::Materialized);
}

inline void FBufferOwner::SetIsMaterialized()
{
	ReferenceCountsAndFlags.fetch_or(SetFlags(EBufferOwnerFlags::Materialized), std::memory_order_release);
}

inline uint32 FBufferOwner::GetTotalRefCount() const
{
	return GetTotalRefCount(ReferenceCountsAndFlags.load(std::memory_order_relaxed));
}

constexpr inline uint32 FBufferOwner::GetTotalRefCount(const uint64 RefCountsAndFlags)
{
	const uint32 SharedRefCount = GetSharedRefCount(RefCountsAndFlags);
	// A non-zero SharedRefCount adds 1 to WeakRefCount to keep the owner alive.
	// Subtract that extra reference when it is present to return an accurate count.
	return GetWeakRefCount(RefCountsAndFlags) + SharedRefCount - !!SharedRefCount;
}

inline bool FBufferOwner::IsUniqueOwnedMutable() const
{
	const uint64 RefCountsAndFlags = ReferenceCountsAndFlags.load(std::memory_order_relaxed);
	return GetTotalRefCount(RefCountsAndFlags) == 1 &&
		(GetFlags(RefCountsAndFlags) & (EBufferOwnerFlags::Owned | EBufferOwnerFlags::Immutable)) == EBufferOwnerFlags::Owned;
}

inline void FBufferOwner::AddSharedReference()
{
	const uint64 PreviousValue = ReferenceCountsAndFlags.fetch_add(SetSharedRefCount(1), std::memory_order_relaxed);
	checkSlow(GetSharedRefCount(PreviousValue) < RefCountMask);
	if (GetSharedRefCount(PreviousValue) == 0)
	{
		AddWeakReference();
	}
}

inline void FBufferOwner::ReleaseSharedReference()
{
	const uint64 PreviousValue = ReferenceCountsAndFlags.fetch_sub(SetSharedRefCount(1), std::memory_order_acq_rel);
	checkSlow(GetSharedRefCount(PreviousValue) > 0);
	if (GetSharedRefCount(PreviousValue) == 1)
	{
		FreeBuffer();
		Data = nullptr;
		Size = 0;
		ReleaseWeakReference();
	}
}

inline bool FBufferOwner::TryAddSharedReference()
{
	for (uint64 Value = ReferenceCountsAndFlags.load(std::memory_order_relaxed);;)
	{
		if (GetSharedRefCount(Value) == 0)
		{
			return false;
		}
		if (ReferenceCountsAndFlags.compare_exchange_weak(Value, Value + SetSharedRefCount(1),
			std::memory_order_relaxed, std::memory_order_relaxed))
		{
			return true;
		}
	}
}

inline void FBufferOwner::AddWeakReference()
{
	const uint64 PreviousValue = ReferenceCountsAndFlags.fetch_add(SetWeakRefCount(1), std::memory_order_relaxed);
	checkSlow(GetWeakRefCount(PreviousValue) < RefCountMask);
}

inline void FBufferOwner::ReleaseWeakReference()
{
	const uint64 PreviousValue = ReferenceCountsAndFlags.fetch_sub(SetWeakRefCount(1), std::memory_order_acq_rel);
	checkSlow(GetWeakRefCount(PreviousValue) > 0);
	if (GetWeakRefCount(PreviousValue) == 1)
	{
		delete this;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::SharedBuffer::Private
{

template <typename FOps>
template <typename FOtherOps>
inline FBufferOwner* TBufferOwnerPtr<FOps>::CopyFrom(const TBufferOwnerPtr<FOtherOps>& Ptr)
{
	FBufferOwner* NewOwner = Ptr.Owner;
	if (NewOwner)
	{
		if constexpr (bIsWeak || !TBufferOwnerPtr<FOtherOps>::bIsWeak)
		{
			FOps::AddRef(*NewOwner);
		}
		else if (!FOps::TryAddRef(*NewOwner))
		{
			NewOwner = nullptr;
		}
	}
	return NewOwner;
}

template <typename FOps>
template <typename FOtherOps>
inline FBufferOwner* TBufferOwnerPtr<FOps>::MoveFrom(TBufferOwnerPtr<FOtherOps>&& Ptr)
{
	FBufferOwner* NewOwner = Ptr.Owner;
	if constexpr (bIsWeak == TBufferOwnerPtr<FOtherOps>::bIsWeak)
	{
		Ptr.Owner = nullptr;
	}
	else if (NewOwner)
	{
		if constexpr (bIsWeak)
		{
			FOps::AddRef(*NewOwner);
		}
		else if (!FOps::TryAddRef(*NewOwner))
		{
			NewOwner = nullptr;
		}
	}
	return NewOwner;
}

template <typename FOps>
inline TBufferOwnerPtr<FOps>::TBufferOwnerPtr(const TBufferOwnerPtr& Ptr)
	: Owner(CopyFrom(Ptr))
{
}

template <typename FOps>
inline TBufferOwnerPtr<FOps>::TBufferOwnerPtr(TBufferOwnerPtr&& Ptr)
	: Owner(MoveFrom(MoveTemp(Ptr)))
{
}

template <typename FOps>
template <typename FOtherOps>
inline TBufferOwnerPtr<FOps>::TBufferOwnerPtr(const TBufferOwnerPtr<FOtherOps>& Ptr)
	: Owner(CopyFrom(Ptr))
{
}

template <typename FOps>
template <typename FOtherOps>
inline TBufferOwnerPtr<FOps>::TBufferOwnerPtr(TBufferOwnerPtr<FOtherOps>&& Ptr)
	: Owner(MoveFrom(MoveTemp(Ptr)))
{
}

template <typename FOps>
inline TBufferOwnerPtr<FOps>::~TBufferOwnerPtr()
{
	FOps::Release(Owner);
}

template <typename FOps>
inline TBufferOwnerPtr<FOps>& TBufferOwnerPtr<FOps>::operator=(const TBufferOwnerPtr& Ptr)
{
	FBufferOwner* const OldOwner = Owner;
	Owner = CopyFrom(Ptr);
	FOps::Release(OldOwner);
	return *this;
}

template <typename FOps>
inline TBufferOwnerPtr<FOps>& TBufferOwnerPtr<FOps>::operator=(TBufferOwnerPtr&& Ptr)
{
	FBufferOwner* const OldOwner = Owner;
	Owner = MoveFrom(MoveTemp(Ptr));
	FOps::Release(OldOwner);
	return *this;
}

template <typename FOps>
template <typename FOtherOps>
inline TBufferOwnerPtr<FOps>& TBufferOwnerPtr<FOps>::operator=(const TBufferOwnerPtr<FOtherOps>& Ptr)
{
	FBufferOwner* const OldOwner = Owner;
	Owner = CopyFrom(Ptr);
	FOps::Release(OldOwner);
	return *this;
}

template <typename FOps>
template <typename FOtherOps>
inline TBufferOwnerPtr<FOps>& TBufferOwnerPtr<FOps>::operator=(TBufferOwnerPtr<FOtherOps>&& Ptr)
{
	FBufferOwner* const OldOwner = Owner;
	Owner = MoveFrom(MoveTemp(Ptr));
	FOps::Release(OldOwner);
	return *this;
}

template <typename FOps>
template <typename FOtherOps>
inline bool TBufferOwnerPtr<FOps>::operator==(const TBufferOwnerPtr<FOtherOps>& Ptr) const
{
	return Owner == Ptr.Owner;
}

template <typename FOps>
template <typename FOtherOps>
inline bool TBufferOwnerPtr<FOps>::operator!=(const TBufferOwnerPtr<FOtherOps>& Ptr) const
{
	return Owner != Ptr.Owner;
}

} // UE::SharedBuffer::Private

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::SharedBuffer::Private
{

template <typename T, typename Allocator>
class TBufferOwnerTArray final : public FBufferOwner
{
public:
	explicit TBufferOwnerTArray(TArray<T, Allocator>&& InArray)
		: Array(MoveTemp(InArray))
	{
		SetBuffer(Array.GetData(), uint64(Array.Num()) * sizeof(T));
		SetIsMaterialized();
		SetIsOwned();
	}

private:
	virtual void FreeBuffer() final
	{
		Array.Empty();
	}

	TArray<T, Allocator> Array;
};

} // UE::SharedBuffer::Private

/** Construct a unique buffer by taking ownership of an array. */
template <typename T, typename Allocator>
[[nodiscard]] inline FUniqueBuffer MakeUniqueBufferFromArray(TArray<T, Allocator>&& Array)
{
	return FUniqueBuffer(new UE::SharedBuffer::Private::TBufferOwnerTArray<T, Allocator>(MoveTemp(Array)));
}

/** Construct a shared buffer by taking ownership of an array. */
template <typename T, typename Allocator>
[[nodiscard]] inline FSharedBuffer MakeSharedBufferFromArray(TArray<T, Allocator>&& Array)
{
	return FSharedBuffer(new UE::SharedBuffer::Private::TBufferOwnerTArray<T, Allocator>(MoveTemp(Array)));
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
