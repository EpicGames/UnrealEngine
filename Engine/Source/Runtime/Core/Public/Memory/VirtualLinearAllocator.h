// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/UniqueLock.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMemory.h"
#include "Misc/AssertionMacros.h"
#include "Templates/AlignmentTemplates.h"

#ifndef UE_ENABLE_VIRTUAL_LINEAR_ALLOCATOR
#define UE_ENABLE_VIRTUAL_LINEAR_ALLOCATOR PLATFORM_HAS_FPlatformVirtualMemoryBlock
#endif

#if UE_ENABLE_VIRTUAL_LINEAR_ALLOCATOR
namespace UE
{

/**
 * A thread-safe linear allocator backed by virtual memory.
 *
 * Reserves address space in one block of ReserveSize and commits pages on demand.
 *
 * Memory is decommitted by Reset() or when AddRef()/Release() are used to track
 * references to the allocator and that reference count drops to 0.
 *
 * Reserved memory is released by the destructor. Use TNoDestroy to avoid freeing.
 */
template <typename MutexType>
class alignas(PLATFORM_CACHE_LINE_SIZE) TVirtualLinearAllocator
{
	using FVirtualMemoryBlock = FPlatformMemory::FPlatformVirtualMemoryBlock;

public:
	// ReserveSize must be a multiple of commit alignment.
	// It is currently safe on every platform to use multiples 64 KiB.
	inline explicit TVirtualLinearAllocator(SIZE_T InReserveSize)
		: ReserveSize(InReserveSize)
	{
		checkf(IsAligned(ReserveSize, FVirtualMemoryBlock::GetCommitAlignment()),
			TEXT("ReserveSize %" SIZE_T_FMT " must be a multiple of commit alignment %" SIZE_T_FMT "."),
			ReserveSize, FVirtualMemoryBlock::GetCommitAlignment());
	}

	inline ~TVirtualLinearAllocator()
	{
		Block.FreeVirtual();
	}

	void* Allocate(SIZE_T Size, uint32 Alignment)
	{
		TUniqueLock<MutexType> Lock(Mutex);

		uint8* Base = (uint8*)Block.GetVirtualPointer();
		if (UNLIKELY(!Base))
		{
			Block = FVirtualMemoryBlock::AllocateVirtual(ReserveSize);
			Base = (uint8*)Block.GetVirtualPointer();
			checkf(Base, TEXT("Failed to reserve %" SIZE_T_FMT " bytes of virtual memory in FVirtualLinearAllocator."), ReserveSize);
		}

		const SIZE_T AlignedOffset = Align(Offset, Alignment);
		const SIZE_T NewOffset = AlignedOffset + Size;
		checkf(NewOffset <= ReserveSize,
			TEXT("Overflowed FVirtualLinearAllocator by %" SIZE_T_FMT " bytes, with reserved size of %" SIZE_T_FMT " bytes."),
			NewOffset - ReserveSize, ReserveSize);

		if (UNLIKELY(CommitSize < NewOffset))
		{
			const SIZE_T NewCommit = Align(NewOffset, FVirtualMemoryBlock::GetCommitAlignment());
			Block.Commit(CommitSize, NewCommit - CommitSize);
			CommitSize = NewCommit;
		}

		Offset = NewOffset;
		return Base + AlignedOffset;
	}

	void Reset()
	{
		TUniqueLock<MutexType> Lock(Mutex);
		// Only reset when the reference count is 0. This will always be the case when reference
		// counting is not being used. When it is being used, AddRef() may be called on another
		// thread between Release() seeing a reference count of 0 and Reset() locking the mutex.
		if (ActiveCount.load(std::memory_order_relaxed) == 0 && CommitSize > 0)
		{
			Block.Decommit(0, CommitSize);
			CommitSize = 0;
			Offset = 0;
		}
	}

	inline void AddRef()
	{
		ActiveCount.fetch_add(1, std::memory_order_relaxed);
	}

	inline void Release()
	{
		if (ActiveCount.fetch_sub(1, std::memory_order_acq_rel) == 1)
		{
			Reset();
		}
	}

private:
	FVirtualMemoryBlock Block;
	SIZE_T ReserveSize = 0;
	SIZE_T CommitSize = 0;
	SIZE_T Offset = 0;
	MutexType Mutex;
	std::atomic<uint32> ActiveCount{0};
};

} // UE
#endif // UE_ENABLE_VIRTUAL_LINEAR_ALLOCATOR
