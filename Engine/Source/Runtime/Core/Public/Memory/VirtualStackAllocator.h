// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMemory.h"
#include "Misc/AssertionMacros.h"
#include "AutoRTFM.h"

#if PLATFORM_HAS_ASAN_INCLUDE
#include <sanitizer/asan_interface.h>
#endif

class FVirtualStackAllocator;

/**
 * Implements a stack-style allocator backed by a dedicated block of virtual memory
 *
 * FVirtualStackAllocator provides the ability to reserve a block of virtual memory and handles the allocation process
 *
 * Freeing memory is handled by FScopedStackAllocatorBookmark which will, upon destruction, free all memory that was allocated after its creation.
 *
 * To use the system, first create a FVirtualStackAllocator with a lifetime longer than any allocations you will make from it.
 * In each scope that you would like be able to bulk free allocations, create an FScopedStackAllocatorBookmark by calling Allocator->CreateScopedBookmark()
 * Make one or more allocations in any nested scope by calling Allocator->Allocate(Size, Alignment)
 * When the FScopedStackAllocatorBookmark is destructed, all memory allocated after that will be freed back to the allocator
 * 
 * FVirtualStackAllocator provides a one page guard at the end of its reservation to protect against overruns. The total usable memory is therefore
 * NextMultipleOf(ReqestedStackSize, SystemPageSize) - SystemPageSize
 */

enum class EVirtualStackAllocatorDecommitMode : uint8
{
	// Default mode, does not decommit pages until the allocator is destroyed
	AllOnDestruction = 0,
	// Decommits all pages once none are in use
	AllOnStackEmpty = 1,
	// Tracks the high water mark and uses it to free excess memory that is not expected to be used again soon
	// This enables us to quickly release memory consumed by a one-off spikey usage pattern while avoiding frequent page management in the steady state
	// See DecommitUnusedPages() for details of the heuristic used
	ExcessOnStackEmpty = 2,
	NumModes
};

struct FScopedStackAllocatorBookmark
{
public:

    CORE_API ~FScopedStackAllocatorBookmark();

private:
    friend FVirtualStackAllocator;

    FScopedStackAllocatorBookmark(void* InRestorePointer, FVirtualStackAllocator* Owner)
		: RestorePointer(InRestorePointer)
		, Owner(Owner) 
	{
	}

    FScopedStackAllocatorBookmark() = delete;
    FScopedStackAllocatorBookmark(const FScopedStackAllocatorBookmark& Other) = delete;
    FScopedStackAllocatorBookmark(FScopedStackAllocatorBookmark&& Other) = delete;
    void operator=(const FScopedStackAllocatorBookmark& Other) = delete;
    void operator=(FScopedStackAllocatorBookmark&& Other) = delete;

    void* RestorePointer;
    FVirtualStackAllocator* Owner;
};

class FVirtualStackAllocator
{
public:

    // RequestedStackSize will be rounded up to a whole number of pages and reserved
    // One page will be earmarked as a guard, so the available memory will be
    // NextMultipleOf(ReqestedStackSize, SystemPageSize) - SystemPageSize
    CORE_API FVirtualStackAllocator(size_t RequestedStackSize, EVirtualStackAllocatorDecommitMode Mode);
    CORE_API ~FVirtualStackAllocator();

	UE_FORCEINLINE_HINT FScopedStackAllocatorBookmark CreateScopedBookmark()
    {
        // Note that this depends on mandatory Return Value Optimization to ensure the memory is only freed once
        return FScopedStackAllocatorBookmark(NextAllocationStart, this);
    }

    CORE_API void* Allocate(size_t Size, size_t Alignment);

    size_t GetAllocatedBytes() const
    {
        return (char*)NextAllocationStart - (char*)VirtualMemory.GetVirtualPointer();
    }

	size_t GetCommittedBytes() const
	{
		return (char*)NextUncommittedPage - (char*)VirtualMemory.GetVirtualPointer();
	}

private:
	FVirtualStackAllocator(const FVirtualStackAllocator& Other) = delete;
	FVirtualStackAllocator(FVirtualStackAllocator&& Other) = delete;
	void operator=(const FVirtualStackAllocator& Other) = delete;
	void operator=(FVirtualStackAllocator&& Other) = delete;

    friend FScopedStackAllocatorBookmark;

	FVirtualStackAllocator();

	inline void Free(void* RestorePointer)
	{
		check(RestorePointer <= NextAllocationStart);

#if PLATFORM_HAS_ASAN_INCLUDE && !UE_AUTORTFM // Disable poisoning when running with AutoRTFM, as poisoning cannot be safely rolled back.
		ASAN_POISON_MEMORY_REGION(RestorePointer, static_cast<uint8*>(NextAllocationStart) - static_cast<uint8*>(RestorePointer));
#endif

		NextAllocationStart = RestorePointer;
		if (UNLIKELY(NextAllocationStart == VirtualMemory.GetVirtualPointer() && DecommitMode != EVirtualStackAllocatorDecommitMode::AllOnDestruction))
		{
			DecommitUnusedPages();
		}
	}

	// Frees unused pages according to the current decommit mode
	void DecommitUnusedPages();

	// VirtualMemory, NextUncommittedPage and RecentHighWaterMark must never be modified from the closed in AutoRTFM.
    FPlatformMemory::FPlatformVirtualMemoryBlock VirtualMemory;
    void* NextUncommittedPage = nullptr;
	void* RecentHighWaterMark = nullptr;

	void* NextAllocationStart = nullptr;

    size_t TotalReservationSize;
    const size_t PageSize;

	EVirtualStackAllocatorDecommitMode DecommitMode = EVirtualStackAllocatorDecommitMode::AllOnDestruction;
};
