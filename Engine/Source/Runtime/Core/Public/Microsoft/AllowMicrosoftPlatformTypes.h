// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS
	#include "Windows/AllowWindowsPlatformTypes.h" // HEADER_UNIT_IGNORE
#else
	#include "Microsoft/AllowMicrosoftPlatformTypesPrivate.h"
#endif
