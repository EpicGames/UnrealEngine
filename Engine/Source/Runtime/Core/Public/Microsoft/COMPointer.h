// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AssertionMacros.h"
#include <Unknwn.h>

#ifdef TEXT_PASTE
	#undef TEXT
	#define TEXT(x) TEXT_PASTE(x)
#endif


/**
 * Smart COM object pointer.
 */
template<typename T>
class TComPtr
{
public:

	typedef T PointerType;

public:

	/** Default constructor (initialized to null). */
	TComPtr()
		:	RawPointer(NULL)
	{ }

	/**
	 * Create and initialize a new instance.
	 *
	 * @param Object The object to point to.
	 */
	TComPtr(PointerType* const Object)
		: RawPointer(Object)
	{
		if (RawPointer)
		{
			RawPointer->AddRef();
		}
	}

	/**
	 * Copy constructor.
	 *
	 * @param Other The instance to copy.
	 */
	TComPtr(const TComPtr<PointerType>& Other)
		: RawPointer(Other.RawPointer)
	{
		if (RawPointer)
		{
			RawPointer->AddRef();
		}
	}

	/**
	 * Move constructor.
	 *
	 * @param Other The instance to move.
	 */
	TComPtr(TComPtr<PointerType>&& Other)
		: RawPointer(Other.RawPointer)
	{	
		Other.RawPointer = NULL;
	}	

	/**
	 * Assignment operator.
	 *
	 * @param Object The object to point to.
	 */
	TComPtr<PointerType>& operator=(PointerType* const Object) 
	{
		if (RawPointer != Object)
		{
			if (Object)
			{
				Object->AddRef();
			}

			if (RawPointer)
			{
				RawPointer->Release();
			}

			RawPointer = Object;
		}

		return *this;
	}

	/**
	 * Copy assignment operator.
	 *
	 * @param Other The instance to copy.
	 */
	TComPtr<PointerType>& operator=(const TComPtr<PointerType>& Other) 
	{
		if (RawPointer != Other.RawPointer)
		{
			if (Other.RawPointer)
			{
				Other->AddRef();
			}

			if (RawPointer)
			{
				RawPointer->Release();
			}

			RawPointer = Other.RawPointer;
		}

		return *this;
	}

	/**
	 * Move assignment operator.
	 *
	 * @param Other The instance to move.
	 */
	TComPtr<PointerType>& operator=(TComPtr<PointerType>&& Other) 
	{			
		if (RawPointer != Other.RawPointer)
		{
			if (RawPointer)
			{
				RawPointer->Release();
			}

			RawPointer = Other.RawPointer;
			Other.RawPointer = NULL;
		}

		return *this;
	}

	/** Destructor. */
	~TComPtr() 
	{
		if (RawPointer)
		{
			RawPointer->Release();
		}
	}

public:

	UE_FORCEINLINE_HINT PointerType** operator&()
	{
		return &(RawPointer);
	}

	inline PointerType* operator->() const 
	{
		check(RawPointer != NULL);
		return RawPointer;
	}

	UE_FORCEINLINE_HINT bool operator==(PointerType* const Object) const
	{
		return RawPointer == Object;
	}

	UE_FORCEINLINE_HINT bool operator!=(PointerType* const Object) const
	{
		return RawPointer != Object;
	}

	UE_FORCEINLINE_HINT operator PointerType*() const
	{
		return RawPointer;
	}

public:

	/**
	 * Set the pointer without adding a reference.
	 *
	 * @param InRawPointer The object to point to.
	 * @see Detach
	 */
	void Attach(PointerType* Object)
	{
		if (RawPointer)
		{
			RawPointer->Release();
		}

		RawPointer = Object;
	}

	/**
	 * Reset the pointer without releasing a reference.
	 *
	 * @see Attach
	 */
	void Detach()
	{
		RawPointer = NULL;
	}

	/**
	 * Initialize this pointer from a COM interface to be queried.
	 *
	 * @param Riid The ID of the interface to be queried.
	 * @param Unknown The object to query the interface from.
	 * @return The result code of the query.
	 */
	HRESULT FromQueryInterface(REFIID Riid, IUnknown* Unknown)
	{
		if (RawPointer)
		{
			RawPointer->Release();
			RawPointer = NULL;
		}

		return Unknown->QueryInterface(Riid, reinterpret_cast<void**>(&(RawPointer)));
	}

	/**
	 * Get raw pointer to the object pointed to.
	 *
	 * @return Pointer to the object, or NULL if not valid.
	 * @see IsValid
	 */
	UE_FORCEINLINE_HINT PointerType* Get() const
	{
		return RawPointer;
	}

	/**
	 * Whether this pointer is pointing to an actual object.
	 *
	 * @return true if the pointer is valid, false otherwise.
	 * @see Get
	 */
	UE_FORCEINLINE_HINT const bool IsValid() const
	{
		return (RawPointer != NULL);
	}

	/** Reset this pointer to null. */
	void Reset()
	{
		if (RawPointer)
		{
			RawPointer->Release();
			RawPointer = NULL;
		}
	}

private:

	/** Pointer to the actual object, if any. */
	PointerType* RawPointer;

	friend UE_FORCEINLINE_HINT uint32 GetTypeHash(const TComPtr<T>& InObjectPtr)
	{
		return GetTypeHash(InObjectPtr.RawPointer);
	}
};
