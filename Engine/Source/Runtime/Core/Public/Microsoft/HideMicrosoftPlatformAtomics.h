// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Not included directly

#if PLATFORM_WINDOWS
	#include "Windows/HideWindowsPlatformAtomics.h"
#else
	#include "Microsoft/HideMicrosoftPlatformAtomicsPrivate.h"
#endif
