// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	MSVCPlatform.h: Setup for any MSVC-using platform
==================================================================================*/

#pragma once

#if _MSC_VER < 1920
	#error "Compiler is expected to support if constexpr"
#endif

#if !defined(__cpp_fold_expressions)
	#error "Compiler is expected to support fold expressions"
#endif

#ifndef PLATFORM_RETURN_ADDRESS
#define PLATFORM_RETURN_ADDRESS()	        _ReturnAddress()
#endif
#ifndef PLATFORM_RETURN_ADDRESS_POINTER
#define PLATFORM_RETURN_ADDRESS_POINTER()	_AddressOfReturnAddress()
#endif

// https://devblogs.microsoft.com/cppblog/improving-the-state-of-debug-performance-in-c/
#if __has_cpp_attribute(msvc::intrinsic)
#define UE_INTRINSIC_CAST [[msvc::intrinsic]]
#endif

// Ensure we can use this builtin - seems to be present on Clang 9, GCC 11 and MSVC 19.26
#ifndef PLATFORM_COMPILER_SUPPORTS_BUILTIN_BITCAST
#define PLATFORM_COMPILER_SUPPORTS_BUILTIN_BITCAST (_MSC_VER >= 1926)
#endif

#ifdef __has_cpp_attribute
	#if __has_cpp_attribute(msvc::lifetimebound)
		#define UE_LIFETIMEBOUND [[msvc::lifetimebound]]
	#endif
#endif

#ifndef UE_NO_PROFILE_ATTRIBUTE
#define UE_NO_PROFILE_ATTRIBUTE
#endif

// MSVC doesn't complain if we pass `nullptr` to memory operations.
#ifndef UE_PLATFORM_REQUIRES_NONNULL_MEMOP_ARGS
#define UE_PLATFORM_REQUIRES_NONNULL_MEMOP_ARGS 0
#endif
