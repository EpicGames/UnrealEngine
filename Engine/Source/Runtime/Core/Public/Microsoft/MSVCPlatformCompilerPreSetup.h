// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Cause cirular dependencies for header units

#if defined(__clang__)
	// UE4 uses a struct packing of 4 for Win32 and 8 for Win64, 
	// and the default packing is 8 for Win32 and 16 for Win64
	#ifdef _WIN64
		#ifndef PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING
			#define PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING \
				_Pragma("pack(push)") \
				_Pragma("pack(16)")
		#endif // PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING
	#else // _WIN64
		#ifndef PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING
			#define PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING \
				_Pragma("pack(push)") \
				_Pragma("pack(8)")
		#endif // PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING
	#endif // _WIN64
	
	#ifndef PRAGMA_POP_PLATFORM_DEFAULT_PACKING
		#define PRAGMA_POP_PLATFORM_DEFAULT_PACKING \
			_Pragma("pack(pop)")
	#endif // PRAGMA_POP_PLATFORM_DEFAULT_PACKING

	#include "Clang/ClangPlatformCompilerPreSetup.h"
#else
	#ifndef PRAGMA_DISABLE_DEPRECATION_WARNINGS
		#define PRAGMA_DISABLE_DEPRECATION_WARNINGS \
			__pragma (warning(push)) \
			__pragma (warning(disable: 4995)) /* 'function': name was marked as #pragma deprecated */ \
			__pragma (warning(disable: 4996)) /* The compiler encountered a deprecated declaration. */
	#endif // PRAGMA_DISABLE_DEPRECATION_WARNINGS

	#ifndef PRAGMA_ENABLE_DEPRECATION_WARNINGS
		#define PRAGMA_ENABLE_DEPRECATION_WARNINGS \
			__pragma (warning(pop))
	#endif // PRAGMA_ENABLE_DEPRECATION_WARNINGS

	#ifndef PRAGMA_DISABLE_BUFFER_OVERRUN_WARNING
		#define PRAGMA_DISABLE_BUFFER_OVERRUN_WARNING \
			__pragma (warning(push)) \
			__pragma (warning(disable:4789)) /* buffer 'identifier' of size N bytes will be overrun; M bytes will be written starting at offset L */
	#endif // PRAGMA_DISABLE_BUFFER_OVERRUN_WARNING

	#ifndef PRAGMA_ENABLE_BUFFER_OVERRUN_WARNING
		#define PRAGMA_ENABLE_BUFFER_OVERRUN_WARNING \
			__pragma (warning(pop))
	#endif // PRAGMA_ENABLE_BUFFER_OVERRUN_WARNING

	#ifndef PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
		#define PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS \
			__pragma (warning(push)) \
			__pragma (warning(disable: 4456)) /* declaration of 'LocalVariable' hides previous local declaration */ \
			__pragma (warning(disable: 4457)) /* declaration of 'LocalVariable' hides function parameter */ \
			__pragma (warning(disable: 4458)) /* declaration of 'LocalVariable' hides class member */ \
			__pragma (warning(disable: 4459)) /* declaration of 'LocalVariable' hides global declaration */ \
			__pragma (warning(disable: 6244)) /* local declaration of <variable> hides previous declaration at <line> of <file> */
	#endif // PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS

	#ifndef PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS
		#define PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS \
			__pragma(warning(pop))
	#endif // PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS

	#ifndef PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS
		#define PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS \
			__pragma (warning(push)) \
			__pragma (warning(disable: 4702)) /* unreachable code */
	#endif // PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS
	
	#ifndef PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS
		#define PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS \
			__pragma (warning(pop))
	#endif // PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS

	#ifndef PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS
		#define PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS \
			__pragma (warning(push)) \
			__pragma (warning(disable: 4244)) /* 'argument': conversion from 'type1' to 'type2', possible loss of data */	\
			__pragma (warning(disable: 4838)) /* 'argument': conversion from 'type1' to 'type2' requires a narrowing conversion */
	#endif // PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

	#ifndef PRAGMA_ENABLE_UNSAFE_TYPECAST_WARNINGS
		#define PRAGMA_ENABLE_UNSAFE_TYPECAST_WARNINGS \
			UE_DEPRECATED_MACRO(5.0, "The PRAGMA_ENABLE_UNSAFE_TYPECAST_WARNINGS macro has been deprecated in favor of PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS. To force enable warnings use PRAGMA_FORCE_UNSAFE_TYPECAST_WARNINGS.")
	#endif // PRAGMA_ENABLE_UNSAFE_TYPECAST_WARNINGS

	#ifndef PRAGMA_FORCE_UNSAFE_TYPECAST_WARNINGS
		#define PRAGMA_FORCE_UNSAFE_TYPECAST_WARNINGS \
			__pragma (warning(push)) \
			__pragma (warning(error: 4244)) /* 'argument': conversion from 'type1' to 'type2', possible loss of data */	\
			__pragma (warning(error: 4838)) /* 'argument': conversion from 'type1' to 'type2' requires a narrowing conversion */
	#endif // PRAGMA_FORCE_UNSAFE_TYPECAST_WARNINGS

	#ifndef PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
		#define PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS \
			__pragma(warning(pop))
	#endif // PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS

	#ifndef PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS
		#define PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS \
			__pragma (warning(push)) \
			__pragma (warning(disable: 4061)) /*  enumerator 'enumator' in switch of enum 'enumeration' is not explicitly handled by a case label */
	#endif // PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

	#ifndef PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS
	#define PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS \
				__pragma(warning(pop))
	#endif // PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS

	#ifndef PRAGMA_DISABLE_UNDEFINED_IDENTIFIER_WARNINGS
		#define PRAGMA_DISABLE_UNDEFINED_IDENTIFIER_WARNINGS \
			__pragma(warning(push)) \
			__pragma(warning(disable: 4668)) /* 'symbol' is not defined as a preprocessor macro, replacing with '0' for 'directives' */
	#endif // PRAGMA_DISABLE_UNDEFINED_IDENTIFIER_WARNINGS

	#ifndef PRAGMA_ENABLE_UNDEFINED_IDENTIFIER_WARNINGS
		#define PRAGMA_ENABLE_UNDEFINED_IDENTIFIER_WARNINGS \
			__pragma(warning(pop))
	#endif // PRAGMA_ENABLE_UNDEFINED_IDENTIFIER_WARNINGS

	#ifndef PRAGMA_DISABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS
		#define PRAGMA_DISABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS \
			__pragma(warning(push)) \
			__pragma(warning(disable: 4265)) /* class' : class has virtual functions, but destructor is not virtual */
	#endif // PRAGMA_DISABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS

	#ifndef PRAGMA_ENABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS
		#define PRAGMA_ENABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS \
			__pragma(warning(pop))
	#endif // PRAGMA_ENABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS

	#ifndef PRAGMA_DISABLE_REORDER_WARNINGS
		#define PRAGMA_DISABLE_REORDER_WARNINGS \
			__pragma(warning(push)) \
			__pragma(warning(disable: 5038)) /* data member 'member1' will be initialized after data member 'member2' data member 'member' will be initialized after base class 'base_class' */
	#endif // PRAGMA_DISABLE_REORDER_WARNINGS

	#ifndef PRAGMA_ENABLE_REORDER_WARNINGS
		#define PRAGMA_ENABLE_REORDER_WARNINGS \
			__pragma(warning(pop))
	#endif // PRAGMA_ENABLE_REORDER_WARNINGS

	#ifndef PRAGMA_DISABLE_REGISTER_WARNINGS
		#define PRAGMA_DISABLE_REGISTER_WARNINGS \
			__pragma(warning(push)) \
			__pragma(warning(disable: 5033)) /* 'register' is no longer a supported storage class */
	#endif // PRAGMA_DISABLE_REGISTER_WARNINGS

	#ifndef PRAGMA_ENABLE_REGISTER_WARNINGS
		#define PRAGMA_ENABLE_REGISTER_WARNINGS \
			__pragma(warning(pop))
	#endif // PRAGMA_ENABLE_REGISTER_WARNINGS

	#ifndef PRAGMA_DISABLE_UNUSED_PRIVATE_FIELDS_WARNINGS
	#define PRAGMA_DISABLE_UNUSED_PRIVATE_FIELDS_WARNINGS
			// MSVC doesn't seem to have a warning similar to -Wunused-private-field
	#endif // PRAGMA_DISABLE_UNUSED_PRIVATE_FIELDS_WARNINGS

	#ifndef PRAGMA_ENABLE_UNUSED_PRIVATE_FIELDS_WARNINGS
	#define PRAGMA_ENABLE_UNUSED_PRIVATE_FIELDS_WARNINGS
			// MSVC doesn't seem to have a warning similar to -Wunused-private-field
	#endif // PRAGMA_ENABLE_UNUSED_PRIVATE_FIELDS_WARNINGS

	#ifndef PRAGMA_DISABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS
	#define PRAGMA_DISABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS
		// MSVC doesn't seem to have a warning similar to -Wuninitialized-const-reference
	#endif // PRAGMA_DISABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS

	#ifndef PRAGMA_ENABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS
	#define PRAGMA_ENABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS
		// MSVC doesn't seem to have a warning similar to -Wuninitialized-const-reference
	#endif // PRAGMA_ENABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS

	#ifndef PRAGMA_DISABLE_ENUM_CONSTEXPR_CONVERSION_WARNINGS
	#define PRAGMA_DISABLE_ENUM_CONSTEXPR_CONVERSION_WARNINGS
		// MSVC doesn't seem to have a warning similar to -Wenum-constexpr-conversion
	#endif // PRAGMA_DISABLE_ENUM_CONSTEXPR_CONVERSION_WARNINGS

	#ifndef PRAGMA_ENABLE_ENUM_CONSTEXPR_CONVERSION_WARNINGS
	#define PRAGMA_ENABLE_ENUM_CONSTEXPR_CONVERSION_WARNINGS
		// MSVC doesn't seem to have a warning similar to -Wenum-constexpr-conversion
	#endif // PRAGMA_ENABLE_ENUM_CONSTEXPR_CONVERSION_WARNINGS

	#ifndef PRAGMA_DISABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY_WARNINGS
	#define PRAGMA_DISABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY_WARNINGS
		// MSVC doesn't seem to have a warning similar to -Wdeprecated-copy-with-user-provided-copy
	#endif // PRAGMA_DISABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY_WARNINGS

	#ifndef PRAGMA_ENABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY_WARNINGS
	#define PRAGMA_ENABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY_WARNINGS
		// MSVC doesn't seem to have a warning similar to -Wdeprecated-copy-with-user-provided-copy
	#endif // PRAGMA_ENABLE_DEPRECATED_COPY_WITH_USER_DEFINED_COPY_WARNINGS

	#ifndef PRAGMA_DISABLE_NONPORTABLE_INCLUDE_PATH_WARNING
	#define PRAGMA_DISABLE_NONPORTABLE_INCLUDE_PATH_WARNING
		// MSVC doesn't seem to have a warning similar to -Wnonportable-include-path
	#endif // PRAGMA_DISABLE_NONPORTABLE_INCLUDE_PATH_WARNING

	#ifndef PRAGMA_ENABLE_NONPORTABLE_INCLUDE_PATH_WARNING
	#define PRAGMA_ENABLE_NONPORTABLE_INCLUDE_PATH_WARNING
		// MSVC doesn't seem to have a warning similar to -Wnonportable-include-path
	#endif // PRAGMA_ENABLE_NONPORTABLE_INCLUDE_PATH_WARNING

	#ifndef PRAGMA_DISABLE_IGNORED_PRAGMAS_WARNINGS
	#define PRAGMA_DISABLE_IGNORED_PRAGMAS_WARNINGS
		// MSVC supports all `#pragma section` actions
	#endif // PRAGMA_DISABLE_IGNORED_PRAGMAS_WARNINGS

	#ifndef PRAGMA_ENABLE_IGNORED_PRAGMAS_WARNINGS
	#define PRAGMA_ENABLE_IGNORED_PRAGMAS_WARNINGS
		// MSVC supports all `#pragma section` actions
	#endif // PRAGMA_ENABLE_IGNORED_PRAGMAS_WARNINGS

	#ifndef PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	#define PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS \
		__pragma(warning(push)) \
		__pragma(warning(disable: 4191)) /* disable the "unsafe conversion from 'FARPROC' to 'blah'" warning */
	#endif // PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

	#ifndef PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	#define PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS \
		__pragma(warning(pop))
	#endif // PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

	#ifndef PRAGMA_POP
		#define PRAGMA_POP \
			__pragma(warning(pop))
	#endif // PRAGMA_POP

	// UE4 uses a struct packing of 4 for Win32 and 8 for Win64, 
	// and the default packing is 8 for Win32 and 16 for Win64
	#ifdef _WIN64
		#ifndef PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING
			#define PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING \
				__pragma(pack(push, 16))
		#endif // PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING
	#else // _WIN64
		#ifndef PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING
			#define PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING \
				__pragma(pack(push, 8))
		#endif // PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING
	#endif // _WIN64
	
	#ifndef PRAGMA_POP_PLATFORM_DEFAULT_PACKING
		#define PRAGMA_POP_PLATFORM_DEFAULT_PACKING \
			__pragma(pack(pop))
	#endif // PRAGMA_POP_PLATFORM_DEFAULT_PACKING

	// Disable common CA warnings around SDK includes
	#ifndef UE_COMPILER_THIRD_PARTY_INCLUDES_START
		#define UE_COMPILER_THIRD_PARTY_INCLUDES_START \
			__pragma(warning(push)) \
			__pragma(warning(disable: 4100))  /* 'identifier': unreferenced formal parameter. */ \
			__pragma(warning(disable: 4125))  /* decimal digit terminates octal escape sequence. */ \
			__pragma(warning(disable: 4267))  /* 'var': conversion from 'size_t' to 'type', possible loss of data. */ \
			__pragma(warning(disable: 4510))  /* '<class>': default constructor could not be generated. */ \
			__pragma(warning(disable: 4610))  /* object '<class>' can never be instantiated - user-defined constructor required. */ \
			__pragma(warning(disable: 4800))  /* Implicit conversion from '<type>' to bool. Possible information loss. */ \
			__pragma(warning(disable: 4834))  /* discarding return value of function with 'nodiscard' attribute. */ \
			__pragma(warning(disable: 4946))  /* reinterpret_cast used between related classes: '<class1>' and '<class2>' */ \
			__pragma(warning(disable: 4996))  /* '<obj>' was declared deprecated. */ \
			__pragma(warning(disable: 6011))  /* Dereferencing NULL pointer '<ptr>'. */ \
			__pragma(warning(disable: 6101))  /* Returning uninitialized memory '<expr>'.  A successful path through the function does not set the named _Out_ parameter. */ \
			__pragma(warning(disable: 6287))  /* Redundant code:  the left and right sub-expressions are identical. */ \
			__pragma(warning(disable: 6308))  /* 'realloc' might return null pointer: assigning null pointer to 'X', which is passed as an argument to 'realloc', will cause the original memory block to be leaked. */ \
			__pragma(warning(disable: 6326))  /* Potential comparison of a constant with another constant. */ \
			__pragma(warning(disable: 6340))  /* Mismatch on sign: Incorrect type passed as parameter in call to function. */ \
			__pragma(warning(disable: 6385))  /* Reading invalid data from '<ptr>':  the readable size is '<num1>' bytes, but '<num2>' bytes may be read. */ \
			__pragma(warning(disable: 6386))  /* Buffer overrun while writing to '<ptr>':  the writable size is '<num1>' bytes, but '<num2>' bytes might be written. */ \
			__pragma(warning(disable: 6553))  /* The annotation for function '<func>' on _Param_(<num>) does not apply to a value type. */ \
			__pragma(warning(disable: 28182)) /* Dereferencing NULL pointer. '<ptr1>' contains the same NULL value as '<ptr2>' did. */ \
			__pragma(warning(disable: 28251)) /* Inconsistent annotation for '<func>': this instance has no annotations. */ \
			__pragma(warning(disable: 28252)) /* Inconsistent annotation for '<func>': return/function has '<annotation>' on the prior instance. */ \
			__pragma(warning(disable: 28253)) /* Inconsistent annotation for '<func>': _Param_(<num>) has '<annotation>' on the prior instance. */ \
			__pragma(warning(disable: 28301)) /* No annotations for first declaration of '<func>'. */ \
			PRAGMA_DISABLE_REORDER_WARNINGS \
			PRAGMA_DISABLE_UNDEFINED_IDENTIFIER_WARNINGS \
			PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS \
			PRAGMA_DISABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS \
			PRAGMA_DISABLE_DEPRECATION_WARNINGS \
			PRAGMA_DISABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS \
			PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS
	#endif // THIRD_PARTY_INCLUDES_START

	#ifndef UE_COMPILER_THIRD_PARTY_INCLUDES_END
		#define UE_COMPILER_THIRD_PARTY_INCLUDES_END \
			PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS \
			PRAGMA_ENABLE_UNINITIALIZED_CONST_REFERENCE_WARNINGS \
			PRAGMA_ENABLE_DEPRECATION_WARNINGS \
			PRAGMA_ENABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS \
			PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS \
			PRAGMA_ENABLE_UNDEFINED_IDENTIFIER_WARNINGS \
			PRAGMA_ENABLE_REORDER_WARNINGS \
			__pragma(warning(pop))
	#endif // THIRD_PARTY_INCLUDES_END

	#define EMIT_CUSTOM_WARNING_AT_LINE(Line, Warning) \
		__pragma(message(WARNING_LOCATION(Line) ": warning C4996: " Warning))

	#define EMIT_CUSTOM_ERROR_AT_LINE(Line, Error) \
		__pragma(message(WARNING_LOCATION(Line) ": error C4996: " Error))

	#ifndef PRAGMA_PUSH_ATTRIBUTE_MINSIZE_FUNCTIONS
		#define PRAGMA_PUSH_ATTRIBUTE_MINSIZE_FUNCTIONS // no-op for now
	#endif // PRAGMA_PUSH_ATTRIBUTE_MINSIZE_FUNCTIONS

	#ifndef PRAGMA_POP_ATTRIBUTE_MINSIZE_FUNCTIONS
		#define PRAGMA_POP_ATTRIBUTE_MINSIZE_FUNCTIONS // no-op for now
	#endif // PRAGMA_POP_ATTRIBUTE_MINSIZE_FUNCTIONS

	#ifndef PGO_LINK_DISABLE_WARNINGS
		#define PGO_LINK_DISABLE_WARNINGS PRAGMA_DISABLE_BUFFER_OVERRUN_WARNING
	#endif

	#ifndef PGO_LINK_ENABLE_WARNINGS
		#define PGO_LINK_ENABLE_WARNINGS PRAGMA_ENABLE_BUFFER_OVERRUN_WARNING
	#endif
#endif // defined(__clang__)
