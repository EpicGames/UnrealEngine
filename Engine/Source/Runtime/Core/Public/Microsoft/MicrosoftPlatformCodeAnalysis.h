// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


// Code analysis features
#if defined( _PREFAST_ ) || defined( PVS_STUDIO ) || defined(__clang_analyzer__)
	#define USING_CODE_ANALYSIS 1
#else
	#define USING_CODE_ANALYSIS 0
#endif

//
// NOTE: To suppress a single occurrence of a code analysis warning:
//
// 		CA_SUPPRESS( <WarningNumber> )
//		...code that triggers warning...
//

//
// NOTE: To disable all code analysis warnings for a section of code (such as include statements
//       for a third party library), you can use the following:
//
// 		#if USING_CODE_ANALYSIS
// 			MSVC_PRAGMA( warning( push ) )
// 			MSVC_PRAGMA( warning( disable : ALL_CODE_ANALYSIS_WARNINGS ) )
// 		#endif	// USING_CODE_ANALYSIS
//
//		<code with warnings>
//
// 		#if USING_CODE_ANALYSIS
// 			MSVC_PRAGMA( warning( pop ) )
// 		#endif	// USING_CODE_ANALYSIS
//

#if USING_CODE_ANALYSIS

// If enabled, add in settings for PVS-Studio Analysis
#include "Analysis/PvsStudioCodeAnalysis.h"

#if !defined(__clang_analyzer__)

	// Input argument
	// Example:  void SetValue( CA_IN bool bReadable );
	#define CA_IN __in

	// Output argument
	// Example:  void FillValue( CA_OUT bool& bWriteable );
	#define CA_OUT __out

	// Specifies that a function parameter may only be read from, never written.
	// NOTE: CA_READ_ONLY is inferred automatically if your parameter is has a const qualifier.
	// Example:  void SetValue( CA_READ_ONLY bool bReadable );
	#define CA_READ_ONLY [Pre(Access=Read)]

	// Specifies that a function parameter may only be written to, never read.
	// Example:  void FillValue( CA_WRITE_ONLY bool& bWriteable );
	#define CA_WRITE_ONLY [Pre(Access=Write)]

	// Incoming pointer parameter must not be NULL and must point to a valid location in memory.
	// Place before a function parameter's type name.
	// Example:  void SetPointer( CA_VALID_POINTER void* Pointer );
	#define CA_VALID_POINTER [Pre(Null=No,Valid=Yes)]

	// Caller must check the return value.  Place before the return value in a function declaration.
	// Example:  CA_CHECK_RETVAL int32 GetNumber();
	#define CA_CHECK_RETVAL [returnvalue:Post(MustCheck=Yes)]

	// Function is expected to never return
	#define CA_NO_RETURN __declspec(noreturn)

	// Suppresses a warning for a single occurrence.  Should be used only for code analysis warnings on Windows platform!
	#define CA_SUPPRESS( WarningNumber ) __pragma( warning( suppress: WarningNumber ) )

	// Tells the code analysis engine to assume the statement to be true.  Useful for suppressing false positive warnings.
#if defined( PVS_STUDIO )
	__declspec(dllimport, noreturn) void CA_AssumeNoReturn();
	#define CA_ASSUME( Expr )  (__builtin_expect(!bool(Expr), 0) ? CA_AssumeNoReturn() : (void)0)
#else
	// NOTE: We use a double operator not here to avoid issues with passing certain class objects directly into __analysis_assume (which may cause a bogus compiler warning)
	#define CA_ASSUME( Expr ) __analysis_assume( !!( Expr ) )
#endif

	// Does a simple 'if (Condition)', but disables warnings about using constants in the condition.  Helps with some macro expansions.
	#define CA_CONSTANT_IF(Condition) __pragma(warning(push)) __pragma(warning(disable:6326)) if (Condition) __pragma(warning(pop))


	//
	// Disable some code analysis warnings that we are NEVER interested in
	//

	// NOTE: Please be conservative about adding new suppressions here!  If you add a suppression, please
	//       add a comment that explains the rationale.

	// We don't use exceptions or care to gracefully handle _alloca() failure.  Also, we wrap _alloca in an
	// appAlloca macro (not inline methods) and don't want to suppress at all call sites.
	#pragma warning(disable : 6255) // warning C6255: _alloca indicates failure by raising a stack overflow exception.  Consider using _malloca instead.

	// This a very common false positive warning (but some cases are not false positives!). Disabling for now so that we
	// can more quickly see the benefits of static analysis on new code.
	#pragma warning(disable : 6102) // warning C6102: Using 'variable' from failed function call at line 'line'.

	// We use this exception handler in Windows code, it may be worth a closer look, but disabling for now
	// so we can get the benefits of analysis on cross-platform code sooner.
	#pragma warning(disable : 6320) // warning C6320: Exception-filter expression is the constant EXCEPTION_EXECUTE_HANDLER. This might mask exceptions that were not intended to be handled.

	// Branching on constants allows us to ensure code paths compile even if they are inactive (eg. if(PLATFORM_FOO){ ... }), and is good practice that should not be discouraged.
	#pragma warning(disable : 6326) // warning C6326 : Potential comparison of a constant with another constant.

	// Likewise, expressions involving constants can also be valuable to check code paths compile.
	#pragma warning(disable : 6240) // warning C6240 : (<expression> && <non-zero constant>) always evaluates to the result of <expression>. Did you intend to use the bitwise-and operator?

	// This is a warning that a static array is exactly 365 members long.  We get false positives because our code gen can make arrays exactly this size.  
	// EX: 365 properties in a UObject will auto gen a table exactly this long.
	#pragma warning(disable : 6393) // warning C6393: A lookup table of size 365 is not sufficient to handle leap years.

#else // defined(__clang_analyzer__)

// A fake function marked with noreturn that acts as a marker for CA_ASSUME to ensure the
// static analyzer doesn't take an analysis path that is assumed not to be navigable.
__declspec(dllimport, noreturn) void CA_AssumeNoReturn();

// Input argument
// Example:  void SetValue( CA_IN bool bReadable );
#define CA_IN

// Output argument
// Example:  void FillValue( CA_OUT bool& bWriteable );
#define CA_OUT

// Specifies that a function parameter may only be read from, never written.
// NOTE: CA_READ_ONLY is inferred automatically if your parameter is has a const qualifier.
// Example:  void SetValue( CA_READ_ONLY bool bReadable );
#define CA_READ_ONLY

// Specifies that a function parameter may only be written to, never read.
// Example:  void FillValue( CA_WRITE_ONLY bool& bWriteable );
#define CA_WRITE_ONLY

// Incoming pointer parameter must not be NULL and must point to a valid location in memory.
// Place before a function parameter's type name.
// Example:  void SetPointer( CA_VALID_POINTER void* Pointer );
#define CA_VALID_POINTER

// Caller must check the return value.  Place before the return value in a function declaration.
// Example:  CA_CHECK_RETVAL int32 GetNumber();
#define CA_CHECK_RETVAL

// Function is expected to never return
#define CA_NO_RETURN __declspec(noreturn)

// Suppresses a warning for a single occurrence.  Should be used only for code analysis warnings on Windows platform!
#define CA_SUPPRESS( WarningNumber )

// Tells the code analysis engine to assume the statement to be true.  Useful for suppressing false positive warnings.
#define CA_ASSUME( Expr )  (__builtin_expect(!bool(Expr), 0) ? CA_AssumeNoReturn() : (void)0)

// Does a simple 'if (Condition)', but disables warnings about using constants in the condition.  Helps with some macro expansions.
#define CA_CONSTANT_IF(Condition) if (Condition)


#endif


#endif