// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"

struct FMicrosoftPlatformCrashContext : public FGenericCrashContext
{
	FMicrosoftPlatformCrashContext(ECrashContextType InType, const TCHAR* InErrorMessage)
		: FGenericCrashContext(InType, InErrorMessage)
	{
	}

	CORE_API void CaptureAllThreadContexts();

	CORE_API virtual void SetPortableCallStack(const uint64* StackTrace, int32 StackTraceDepth) override;
	CORE_API virtual void AddPortableThreadCallStacks(TConstArrayView<FThreadCallStack> Threads) override;
	CORE_API virtual void AddPortableThreadCallStack(uint32 ThreadId, const TCHAR* ThreadName, const uint64* StackFrames, int32 NumStackFrames) override;

protected:
	// Helpers
	typedef TArray<void*, TInlineAllocator<128>> FModuleHandleArray;

	static CORE_API void GetProcModuleHandles(const FProcHandle& Process, FModuleHandleArray& OutHandles);

	static CORE_API void ConvertProgramCountersToStackFrames(
		const FProcHandle& Process,
		const FModuleHandleArray& SortedModuleHandles,
		const uint64* ProgramCounters,
		int32 NumPCs,
		TArray<FCrashStackFrame>& OutStackFrames);

	CORE_API virtual bool GetPlatformAllThreadContextsString(FString& OutStr) const override;

	static CORE_API void AddThreadContextString(
		uint32 CrashedThreadId,
		uint32 ThreadId,
		const FString& ThreadName,
		const TArray<FCrashStackFrame>& StackFrames,
		FString& OutStr);

};
