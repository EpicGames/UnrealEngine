// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "Math/UnrealPlatformMathSSE4.h"

/**
* Microsoft base implementation of the Math OS functions
**/
struct FMicrosoftPlatformMathBase : public TUnrealPlatformMathSSE4Base<FGenericPlatformMath>
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	static UE_FORCEINLINE_HINT bool IsNaN( float A ) { return _isnan(A) != 0; }
	static UE_FORCEINLINE_HINT bool IsNaN(double A) { return _isnan(A) != 0; }
	static UE_FORCEINLINE_HINT bool IsFinite( float A ) { return _finite(A) != 0; }
	static UE_FORCEINLINE_HINT bool IsFinite(double A) { return _finite(A) != 0; }

	#pragma intrinsic(_BitScanReverse)

	static inline uint32 FloorLog2(uint32 Value)
	{
		// Use BSR to return the log2 of the integer
		// return 0 if value is 0
		unsigned long BitIndex;
		return _BitScanReverse(&BitIndex, Value) ? BitIndex : 0;
	}

	static inline uint32 FloorLog2NonZero(uint32 Value)
	{
		unsigned long BitIndex = 0;
		_BitScanReverse(&BitIndex, Value);
		return BitIndex;
	}

	static constexpr inline uint8 CountLeadingZeros8(uint8 Value)
	{
		// return 8 if value was 0

		UE_IF_CONSTEVAL
		{
			return FGenericPlatformMath::CountLeadingZeros8(Value);
		}
		else
		{
			unsigned long BitIndex;
			_BitScanReverse(&BitIndex, uint32(Value) * 2 + 1);
			return uint8(8 - BitIndex);
		}
	}

	static constexpr inline uint32 CountTrailingZeros(uint32 Value)
	{

		UE_IF_CONSTEVAL
		{
			return FGenericPlatformMath::CountTrailingZeros(Value);
		}
		else
		{
			// return 32 if value was 0
			unsigned long BitIndex;	// 0-based, where the LSB is 0 and MSB is 31
			return _BitScanForward(&BitIndex, Value) ? BitIndex : 32;
		}
	}

	static constexpr inline uint32 CeilLogTwo( uint32 Arg )
	{
		// if Arg is 0, change it to 1 so that we return 0
		Arg = Arg ? Arg : 1;
		return 32 - CountLeadingZeros(Arg - 1);
	}

	static UE_FORCEINLINE_HINT uint32 RoundUpToPowerOfTwo(uint32 Arg)
	{
		return 1u << CeilLogTwo(Arg);
	}

	static UE_FORCEINLINE_HINT uint64 RoundUpToPowerOfTwo64(uint64 Arg)
	{
		return uint64(1) << CeilLogTwo64(Arg);
	}

	#pragma intrinsic(_BitScanReverse64)

	static inline uint64 FloorLog2_64(uint64 Value)
	{
		unsigned long BitIndex;
		return _BitScanReverse64(&BitIndex, Value) ? BitIndex : 0;
	}

	static inline uint64 FloorLog2NonZero_64(uint64 Value)
	{
		unsigned long BitIndex = 0;
		_BitScanReverse64(&BitIndex, Value);
		return BitIndex;
	}

	static inline uint64 CeilLogTwo64(uint64 Arg)
	{
		// if Arg is 0, change it to 1 so that we return 0
		Arg = Arg ? Arg : 1;
		return 64 - CountLeadingZeros64(Arg - 1);
	}

	static constexpr inline uint64 CountLeadingZeros64(uint64 Value)
	{

		UE_IF_CONSTEVAL
		{
			return FGenericPlatformMath::CountLeadingZeros64(Value);
		}
		else
		{
			//https://godbolt.org/z/Ejh5G4vPK	
			// return 64 if value if was 0
			unsigned long BitIndex;
			if (!_BitScanReverse64(&BitIndex, Value)) BitIndex = -1;
			return 63 - BitIndex;
		}
	}

	static constexpr inline uint64 CountTrailingZeros64(uint64 Value)
	{

		UE_IF_CONSTEVAL
		{
			return FGenericPlatformMath::CountTrailingZeros64(Value);
		}
		else
		{
			// return 64 if Value is 0
			unsigned long BitIndex;	// 0-based, where the LSB is 0 and MSB is 63
			return _BitScanForward64(&BitIndex, Value) ? BitIndex : 64;
		}
	}

	static constexpr inline uint32 CountLeadingZeros(uint32 Value)
	{

		UE_IF_CONSTEVAL
		{
			return FGenericPlatformMath::CountLeadingZeros(Value);
		}
		else
		{
			// return 32 if value is zero
			unsigned long BitIndex;
			_BitScanReverse64(&BitIndex, uint64(Value) * 2 + 1);
			return 32 - BitIndex;
		}
	}

	[[nodiscard]] static constexpr inline int32 CountBits(uint32 Bits)
	{
		UE_IF_CONSTEVAL
		{
			return FGenericPlatformMath::CountBits(Bits);
		}
		else
		{
			return (int32)__popcnt(Bits);
		}
	}

	[[nodiscard]] static constexpr inline int32 CountBits64(uint64 Bits)
	{
		UE_IF_CONSTEVAL
		{
			return FGenericPlatformMath::CountBits64(Bits);
		}
		else
		{
			return (int32)__popcnt64(Bits);
		}
	}

	template <UE::CIntegral T>
		requires (sizeof(T) > 4)
	[[nodiscard]] UE_REWRITE static constexpr int32 CountBits(T Bits)
	{
		UE_STATIC_DEPRECATE(5.8, sizeof(T) > 4, "CountBits for 64-bit values has been deprecated, please use CountBits64 instead.");
		return CountBits64(Bits);
	}

#endif
};
