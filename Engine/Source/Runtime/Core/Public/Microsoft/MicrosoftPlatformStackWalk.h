// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"

/**
 * Microsoft-shared implementation of the stack walking.
 **/
struct FMicrosoftPlatformStackWalk
	: public FGenericPlatformStackWalk
{

protected:
	// Extract debug info for a module from the module header in memory. 
	// Can directly read the information even when the current target can't load the symbols itself or use certain DbgHelp APIs.
	static CORE_API bool ExtractInfoFromModule(void* ProcessHandle, void* ModuleHandle, FStackWalkModuleInfo& OutInfo);

	//
	// If the thread is not the calling thread, it should already have been suspended by the calling code.

	/**
	 * Helper for Microsoft platforms to capture a stacktrace for a specific thread.
	 * If the thread is not the calling thread, it should already have been suspended by the calling code.
	 *
	 * @param	OutBackTrace		Array to write backtrace to
	 * @param	MaxDepth			Maximum depth to walk - needs to be less than or equal to array size
	 * @param	Context				Pointer to a CONTEXT structure containing thread context info (registers, instruction ptr, etc)
	 * @param	ThreadHandle		HANDLE for the thread whose stack should be walked.
	 * @param	OutDepth			Number of stack frames returned.
	 * @return	EXCEPTION_EXECUTE_HANDLER
	 */
	static CORE_API int32 CaptureStackTraceInternal(uint64* OutBacktrace, uint32 MaxDepth, void* Context, void* ThreadHandle, uint32* OutDepth);
};
