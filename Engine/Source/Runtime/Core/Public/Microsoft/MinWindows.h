// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS
	#include "Windows/MinWindows.h" // HEADER_UNIT_IGNORE
#else
	#if !defined(WINDOWS_H_WRAPPER_GUARD)
	#pragma message("WARNING: do not include Microsoft/MinWindows.h directly. Use Microsoft/WindowsHWrapper.h instead")
	#endif

	#include "Microsoft/MinWindowsPrivate.h"
#endif
