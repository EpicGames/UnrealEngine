// Copyright Epic Games, Inc. All Rights Reserved.

//=====================================================================================================================
// Implementation of a minimal subset of the Windows API required for inline function definitions and platform-specific
// interfaces in Core. Use of this file allows avoiding including large platform headers in the public engine API.
//
// Win32 API functions are declared in the "Windows" namespace to avoid conflicts if the real Windows.h is included 
// later, but are mapped to the same imported symbols by the linker due to C-style linkage.
//=====================================================================================================================

#pragma once

#include "CoreTypes.h"

#ifdef __clang__
	#define MINIMAL_WINDOWS_API CORE_API
#else
	#define MINIMAL_WINDOWS_API extern "C" __declspec(dllimport)
#endif

// The 10.0.18362.0 SDK introduces an error if the packing isn't the default for the platform.
PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING

// Undefine Windows types that we're going to redefine. This should be done by HideWindowsPlatformTypes.h after including any system headers, 
// but it's less friction to just undefine them here.
#pragma push_macro("TRUE")
#pragma push_macro("FALSE")
#undef TRUE
#undef FALSE

// Use strongly typed handles
#ifndef STRICT
#define STRICT
#endif

// With STRICT enabled, handles are implemented as opaque struct pointers. We can prototype these structs here and 
// typedef them under the Windows namespace below. Since typedefs are only aliases rather than types in their own 
// right, this allows using handles from the Windows:: namespace interchangably with their native definition.
struct HINSTANCE__;
struct HWND__;
struct HKEY__;
struct HDC__;
struct HICON__;
struct _RTL_SRWLOCK;

// Other types
struct tagPROCESSENTRY32W;
struct _EXCEPTION_POINTERS;
struct _OVERLAPPED;
struct _RTL_CRITICAL_SECTION;
union _LARGE_INTEGER;

// Global constants
#define WINDOWS_MAX_PATH 260
#define WINDOWS_PF_COMPARE_EXCHANGE128	14

// Standard calling convention for Win32 functions
#define WINAPI __stdcall

// Minimal subset of the Windows API required for interfaces and inline functions
namespace Windows
{
	// Typedefs for basic Windows types
	typedef int32 BOOL;
	typedef unsigned char BYTE;
	typedef BYTE  BOOLEAN;
	typedef unsigned long DWORD;
	typedef DWORD* LPDWORD;
	typedef long LONG;
	typedef long* LPLONG;
	typedef int64 LONGLONG;
	typedef LONGLONG* LPLONGLONG;
	typedef void* LPVOID;
	typedef const void* LPCVOID;
	typedef const wchar_t* LPCTSTR;

	// Typedefs for standard handles
	typedef void* HANDLE;
	typedef HINSTANCE__* HINSTANCE;
	typedef HINSTANCE HMODULE;
	typedef HWND__* HWND;
	typedef HKEY__* HKEY;
	typedef HDC__* HDC;
	typedef HICON__* HICON;
	typedef HICON__* HCURSOR;

	// Other typedefs
	typedef tagPROCESSENTRY32W PROCESSENTRY32;
	typedef _EXCEPTION_POINTERS* LPEXCEPTION_POINTERS;
	typedef _RTL_CRITICAL_SECTION* LPCRITICAL_SECTION;
	typedef _OVERLAPPED* LPOVERLAPPED;
	typedef _LARGE_INTEGER* LPLARGE_INTEGER;

	typedef _RTL_SRWLOCK RTL_SRWLOCK, *PRTL_SRWLOCK;
	typedef RTL_SRWLOCK *PSRWLOCK;

	typedef void (*PFLS_CALLBACK_FUNCTION) (void* lpFlsData);


	// Opaque SRWLOCK structure
	struct SRWLOCK
	{
		void* Ptr;
	};

	// Constants
	inline constexpr BOOL TRUE = 1;
	inline constexpr BOOL FALSE = 0;

	// Modules
	MINIMAL_WINDOWS_API HMODULE WINAPI LoadLibraryW(LPCTSTR lpFileName);
	MINIMAL_WINDOWS_API BOOL WINAPI FreeLibrary(HMODULE hModule);

	// Critical sections
	MINIMAL_WINDOWS_API void WINAPI InitializeCriticalSection(LPCRITICAL_SECTION lpCriticalSection);
	MINIMAL_WINDOWS_API BOOL WINAPI InitializeCriticalSectionAndSpinCount(LPCRITICAL_SECTION lpCriticalSection, DWORD dwSpinCount);
	MINIMAL_WINDOWS_API DWORD WINAPI SetCriticalSectionSpinCount(LPCRITICAL_SECTION lpCriticalSection, DWORD dwSpinCount);
	MINIMAL_WINDOWS_API BOOL WINAPI TryEnterCriticalSection(LPCRITICAL_SECTION lpCriticalSection);
	MINIMAL_WINDOWS_API void WINAPI EnterCriticalSection(LPCRITICAL_SECTION lpCriticalSection);
	MINIMAL_WINDOWS_API void WINAPI LeaveCriticalSection(LPCRITICAL_SECTION lpCriticalSection);
	MINIMAL_WINDOWS_API void WINAPI DeleteCriticalSection(LPCRITICAL_SECTION lpCriticalSection);

	MINIMAL_WINDOWS_API void WINAPI InitializeSRWLock(PSRWLOCK SRWLock);
	MINIMAL_WINDOWS_API void WINAPI AcquireSRWLockShared(PSRWLOCK SRWLock);
	MINIMAL_WINDOWS_API void WINAPI ReleaseSRWLockShared(PSRWLOCK SRWLock);
	MINIMAL_WINDOWS_API void WINAPI AcquireSRWLockExclusive(PSRWLOCK SRWLock);
	MINIMAL_WINDOWS_API void WINAPI ReleaseSRWLockExclusive(PSRWLOCK SRWLock);
	MINIMAL_WINDOWS_API BOOLEAN WINAPI TryAcquireSRWLockExclusive(PSRWLOCK SRWLock);
	MINIMAL_WINDOWS_API BOOLEAN WINAPI TryAcquireSRWLockShared(PSRWLOCK SRWLock);

	UE_FORCEINLINE_HINT void WINAPI InitializeSRWLock(SRWLOCK* SRWLock)
	{
		InitializeSRWLock((PSRWLOCK)SRWLock);
	}

	UE_FORCEINLINE_HINT BOOLEAN WINAPI TryAcquireSRWLockExclusive(SRWLOCK* SRWLock)
	{
		return TryAcquireSRWLockExclusive((PSRWLOCK)SRWLock);
	}

	UE_FORCEINLINE_HINT BOOLEAN WINAPI TryAcquireSRWLockShared(SRWLOCK* SRWLock)
	{
		return TryAcquireSRWLockShared((PSRWLOCK)SRWLock);
	}
	
	UE_FORCEINLINE_HINT void WINAPI AcquireSRWLockShared(SRWLOCK* SRWLock)
	{
		AcquireSRWLockShared((PSRWLOCK)SRWLock);
	}

	UE_FORCEINLINE_HINT void WINAPI ReleaseSRWLockShared(SRWLOCK* SRWLock)
	{
		ReleaseSRWLockShared((PSRWLOCK)SRWLock);
	}

	UE_FORCEINLINE_HINT void WINAPI AcquireSRWLockExclusive(SRWLOCK* SRWLock)
	{
		AcquireSRWLockExclusive((PSRWLOCK)SRWLock);
	}

	UE_FORCEINLINE_HINT void WINAPI ReleaseSRWLockExclusive(SRWLOCK* SRWLock)
	{
		ReleaseSRWLockExclusive((PSRWLOCK)SRWLock);
	}

	// I/O
	MINIMAL_WINDOWS_API BOOL WINAPI ConnectNamedPipe(HANDLE hNamedPipe, LPOVERLAPPED lpOverlapped);
	MINIMAL_WINDOWS_API BOOL WINAPI GetOverlappedResult(HANDLE hFile, LPOVERLAPPED lpOverlapped, LPDWORD lpNumberOfBytesTransferred, BOOL bWait);
	MINIMAL_WINDOWS_API BOOL WINAPI WriteFile(HANDLE hFile, LPCVOID lpBuffer, DWORD nNumberOfBytesToWrite, LPDWORD lpNumberOfBytesWritten, LPOVERLAPPED lpOverlapped);
	MINIMAL_WINDOWS_API BOOL WINAPI ReadFile(HANDLE hFile, LPVOID lpBuffer, DWORD nNumberOfBytesToRead, LPDWORD lpNumberOfBytesRead, LPOVERLAPPED lpOverlapped);
	MINIMAL_WINDOWS_API BOOL WINAPI CancelIoEx(HANDLE hFile, LPOVERLAPPED lpOverlapped);
	// Timing
	MINIMAL_WINDOWS_API BOOL WINAPI QueryPerformanceCounter(LPLARGE_INTEGER Cycles);

	// Thread-local storage functions
	MINIMAL_WINDOWS_API DWORD WINAPI GetCurrentThreadId();
	MINIMAL_WINDOWS_API DWORD WINAPI TlsAlloc();
	MINIMAL_WINDOWS_API LPVOID WINAPI TlsGetValue(DWORD dwTlsIndex);
	MINIMAL_WINDOWS_API BOOL WINAPI TlsSetValue(DWORD dwTlsIndex, LPVOID lpTlsValue);
	MINIMAL_WINDOWS_API BOOL WINAPI TlsFree(DWORD dwTlsIndex);

	// Fiber-local storage functions
	MINIMAL_WINDOWS_API DWORD WINAPI FlsAlloc(PFLS_CALLBACK_FUNCTION lpCallback);
	MINIMAL_WINDOWS_API LPVOID WINAPI FlsGetValue(DWORD dwFlsIndex);
	MINIMAL_WINDOWS_API BOOL WINAPI FlsSetValue(DWORD dwFlsIndex, LPVOID lpFlsValue);
	MINIMAL_WINDOWS_API BOOL WINAPI FlsFree(DWORD dwTlsIndex);

	// System
	MINIMAL_WINDOWS_API BOOL WINAPI IsProcessorFeaturePresent(DWORD ProcessorFeature);

	// For structures which are opaque
	struct CRITICAL_SECTION { void* Opaque1[1]; long Opaque2[2]; void* Opaque3[3]; };
	union LARGE_INTEGER { struct { DWORD LowPart; LONG HighPart; }; LONGLONG QuadPart; };

	struct OVERLAPPED
	{
		unsigned long long Internal;
		unsigned long long InternalHigh;
		void* Pointer;
		HANDLE hEvent;
	};

	UE_FORCEINLINE_HINT BOOL WINAPI ConnectNamedPipe(HANDLE hNamedPipe, OVERLAPPED* lpOverlapped)
	{
		return ConnectNamedPipe(hNamedPipe, (LPOVERLAPPED)lpOverlapped);
	}

	UE_FORCEINLINE_HINT BOOL WINAPI GetOverlappedResult(HANDLE hFile, OVERLAPPED* lpOverlapped, LPDWORD lpNumberOfBytesTransferred, BOOL bWait)
	{
		return GetOverlappedResult(hFile, (LPOVERLAPPED)lpOverlapped, lpNumberOfBytesTransferred, bWait);
	}

	UE_FORCEINLINE_HINT BOOL WINAPI WriteFile(HANDLE hFile, LPCVOID lpBuffer, DWORD nNumberOfBytesToWrite, LPDWORD lpNumberOfBytesWritten, OVERLAPPED* lpOverlapped)
	{
		return WriteFile(hFile, lpBuffer, nNumberOfBytesToWrite, lpNumberOfBytesWritten, (LPOVERLAPPED)lpOverlapped);
	}

	UE_FORCEINLINE_HINT BOOL WINAPI ReadFile(HANDLE hFile, LPVOID lpBuffer, DWORD nNumberOfBytesToRead, LPDWORD lpNumberOfBytesRead, OVERLAPPED* lpOverlapped)
	{
		return ReadFile(hFile, lpBuffer, nNumberOfBytesToRead, lpNumberOfBytesRead, (LPOVERLAPPED)lpOverlapped);
	}

	UE_FORCEINLINE_HINT BOOL WINAPI CancelIoEx(HANDLE hFile, OVERLAPPED* lpOverlapped)
	{
		return CancelIoEx(hFile, (LPOVERLAPPED)lpOverlapped);
	}

	UE_FORCEINLINE_HINT void WINAPI InitializeCriticalSection(CRITICAL_SECTION* lpCriticalSection)
	{
		InitializeCriticalSection((LPCRITICAL_SECTION)lpCriticalSection);
	}

	UE_FORCEINLINE_HINT BOOL WINAPI InitializeCriticalSectionAndSpinCount(CRITICAL_SECTION* lpCriticalSection, DWORD dwSpinCount)
	{
		return InitializeCriticalSectionAndSpinCount((LPCRITICAL_SECTION)lpCriticalSection, dwSpinCount);
	}

	UE_FORCEINLINE_HINT DWORD WINAPI SetCriticalSectionSpinCount(CRITICAL_SECTION* lpCriticalSection, DWORD dwSpinCount)
	{
		return SetCriticalSectionSpinCount((LPCRITICAL_SECTION)lpCriticalSection, dwSpinCount);
	}

	UE_FORCEINLINE_HINT BOOL WINAPI TryEnterCriticalSection(CRITICAL_SECTION* lpCriticalSection)
	{
		return TryEnterCriticalSection((LPCRITICAL_SECTION)lpCriticalSection);
	}

	UE_FORCEINLINE_HINT void WINAPI EnterCriticalSection(CRITICAL_SECTION* lpCriticalSection)
	{
		EnterCriticalSection((LPCRITICAL_SECTION)lpCriticalSection);
	}

	UE_FORCEINLINE_HINT void WINAPI LeaveCriticalSection(CRITICAL_SECTION* lpCriticalSection)
	{
		LeaveCriticalSection((LPCRITICAL_SECTION)lpCriticalSection);
	}

	UE_FORCEINLINE_HINT void WINAPI DeleteCriticalSection(CRITICAL_SECTION* lpCriticalSection)
	{
		DeleteCriticalSection((LPCRITICAL_SECTION)lpCriticalSection);
	}

	UE_FORCEINLINE_HINT BOOL WINAPI QueryPerformanceCounter(LARGE_INTEGER* Cycles)
	{
		return QueryPerformanceCounter((LPLARGE_INTEGER)Cycles);
	}
}

// Restore the definitions for TRUE and FALSE
#pragma pop_macro("FALSE")
#pragma pop_macro("TRUE")

// Restore the packing setting
PRAGMA_POP_PLATFORM_DEFAULT_PACKING
