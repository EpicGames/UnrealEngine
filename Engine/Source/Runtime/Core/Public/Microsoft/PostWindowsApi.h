// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Not included directly

#if PLATFORM_WINDOWS
	#include "Windows/PostWindowsApi.h"
#else
	// this file should only be included from WindowsHWrapper.h
	#if !defined(WINDOWS_H_WRAPPER_GUARD) 
	#pragma message("WARNING: do not include Microsoft/PostWindowsApi.h directly. Use Microsoft/WindowsHWrapper.h or Microsoft/HideMicrosoftPlatformTypes.h instead") 
	#endif

	#include "Microsoft/PostWindowsApiPrivate.h"
#endif
