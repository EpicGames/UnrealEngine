// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS
	#include "Windows/WindowsHWrapper.h" // HEADER_UNIT_IGNORE
#else
	#include "CoreTypes.h"
	#include "HAL/PlatformMemory.h"

	#if defined(WINDOWS_H_WRAPPER_GUARD)
	#error WINDOWS_H_WRAPPER_GUARD already defined
	#endif
	#define WINDOWS_H_WRAPPER_GUARD
	
	#include "Microsoft/PreWindowsApiPrivate.h"
	#ifndef STRICT
	#define STRICT
	#endif
	#include "Microsoft/MinWindowsPrivate.h"
	#include "Microsoft/PostWindowsApiPrivate.h"
	
	#undef WINDOWS_H_WRAPPER_GUARD
#endif
