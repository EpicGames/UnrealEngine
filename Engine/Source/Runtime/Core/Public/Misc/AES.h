// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"

// AES-256 implementation - using ECB mode for multiple blocks

// The currently implemented approach has the shortcoming that it encrypts and decrypts each
// 128-bit block separately. If the plaintext contains identical 128-byte blocks, the blocks
// will be encrypted identically. This makes some of the plaintext structure visible in the
// ciphertext, even to someone who does not have the key.

// DO NOT USE this functionality for any new place where you might need encryption. Because
// current code is meant for keeping backwards compatiblity with existing data. Better way
// to use AES would be integrated with authentication, for example, in AES-GCM mode.

struct FAES
{
	static constexpr uint32 AESBlockSize = 16;

	/** 
	 * Class representing a 256 bit AES key
	 */
	struct FAESKey
	{
		static constexpr int32 KeySize = 32;

		uint8 Key[KeySize];

		FAESKey()
		{
			Reset();
		}

		bool IsValid() const
		{
			uint32* Words = (uint32*)Key;
			for (int32 Index = 0; Index < KeySize / 4; ++Index)
			{
				if (Words[Index] != 0)
				{
					return true;
				}
			}
			return false;
		}

		void Reset()
		{
			FMemory::Memset(Key, 0, KeySize);
		}

		bool operator == (const FAESKey& Other) const
		{
			return FMemory::Memcmp(Key, Other.Key, KeySize) == 0;
		}
	};

	/**
	* Encrypts a chunk of data using a specific key
	*
	* @param Contents the buffer to encrypt
	* @param NumBytes the size of the buffer
	* @param Key An FAESKey object containing the encryption key
	*/
	static CORE_API void EncryptData(uint8* Contents, uint64 NumBytes, const FAESKey& Key);

	/**
	 * Encrypts a chunk of data using a specific key
	 *
	 * @param Contents the buffer to encrypt
	 * @param NumBytes the size of the buffer
	 * @param Key a null terminated string that is a 32 bytes long
	 */
	static CORE_API void EncryptData(uint8* Contents, uint64 NumBytes, const ANSICHAR* Key);

	/**
	* Encrypts a chunk of data using a specific key
	*
	* @param Contents the buffer to encrypt
	* @param NumBytes the size of the buffer
	* @param Key a byte array that is a 32 byte length
	* @param NumKeyBytes length of Key byte array, must be 32
	*/
	static CORE_API void EncryptData(uint8* Contents, uint64 NumBytes, const uint8* KeyBytes, uint32 NumKeyBytes);

	/**
	* Decrypts a chunk of data using a specific key
	*
	* @param Contents the buffer to encrypt
	* @param NumBytes the size of the buffer
	* @param Key An FAESKey object containing the decryption key
	*/
	static CORE_API void DecryptData(uint8* Contents, uint64 NumBytes, const FAESKey& Key);

	/**
	 * Decrypts a chunk of data using a specific key
	 *
	 * @param Contents the buffer to encrypt
	 * @param NumBytes the size of the buffer
	 * @param Key a null terminated string that is a 32 bytes long
	 */
	static CORE_API void DecryptData(uint8* Contents, uint64 NumBytes, const ANSICHAR* Key);

	/**
	* Decrypts a chunk of data using a specific key
	*
	* @param Contents the buffer to encrypt
	* @param NumBytes the size of the buffer
	* @param Key a byte array that is a 32 byte length
	* @param NumKeyBytes length of Key byte array, must be 32
	*/
	static CORE_API void DecryptData(uint8* Contents, uint64 NumBytes, const uint8* KeyBytes, uint32 NumKeyBytes);
};
