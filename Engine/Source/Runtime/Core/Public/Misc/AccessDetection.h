// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"

namespace UE { namespace AccessDetection {

enum class EType : uint32
{
	None			= 0,
	File			= 1 << 0, 
	Ini UE_DEPRECATED(5.4, "Use FConfigAccessTracking in ConfigAccessTracking instead") = 1 << 1,
	CVar 			= 1 << 2,
};
ENUM_CLASS_FLAGS(EType);

#if WITH_EDITOR

class FScope;

namespace Private
{
	extern CORE_API volatile int32 GNumLiveScopes;

	CORE_API void ReportCurrentThreadAccess(EType Type);
	CORE_API void SetCurrentThreadScope(FScope* Scope);
}

FORCEINLINE void ReportAccess(EType Type)
{
	// Note unsynchronized access is fine - we only need to see writes by current thread
	if (Private::GNumLiveScopes > 0)
	{
		Private::ReportCurrentThreadAccess(Type);
	}
}

/** Detects access to global core systems while running deterministic code that should be isolated from the local environment */
class FScope
{
public:
	FScope() { Private::SetCurrentThreadScope(this); }
	~FScope() { Private::SetCurrentThreadScope(nullptr); }
	
	void AddAccess(EType Type) { AccessedTypes |= Type; }
	EType GetAccesses() const { return AccessedTypes; }

private:
	EType AccessedTypes = EType::None;
};

#else

FORCEINLINE void ReportAccess(EType Type) {}

#endif

}}
