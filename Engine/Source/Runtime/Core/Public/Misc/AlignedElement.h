// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include <type_traits>

namespace UE::Core
{
	// Helper class to wrap a type for alignment in containers
	template <typename T, uint32 Alignment>
	struct alignas(Alignment) TAlignedElement
	{
		static_assert(!std::is_reference_v<T>);
		
		T Value;

		UE_REWRITE constexpr TAlignedElement& operator=(T&& Other)
		{
			Value = MoveTemp(Other);
			return *this;
		}

		UE_REWRITE constexpr TAlignedElement& operator=(const T& Other)
		{
			Value = Other;
			return *this;
		}
		
		UE_REWRITE constexpr operator T&&()&&
		{
			return MoveTemp(Value);
		}
		
		UE_REWRITE constexpr operator const T&() const&
		{
			return Value;
		}
		
		UE_REWRITE constexpr operator T&()&
		{
			return Value;
		}
	};

	template <typename T, uint32 Alignment> requires(std::is_class_v<T> && !std::is_final_v<T>)
	struct alignas(Alignment) TAlignedElement<T, Alignment>: T
	{
		UE_REWRITE constexpr TAlignedElement& operator=(T&& Other)
		{
			T::operator=(MoveTemp(Other));
			return *this;
		}
		
		UE_REWRITE constexpr TAlignedElement& operator=(const T& Other)
		{
			T::operator=(Other);
			return *this;
		}
	};
}
