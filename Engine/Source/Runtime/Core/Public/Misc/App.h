// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "Misc/AppTime.h"
#include "Misc/Build.h"
#include "Misc/CString.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreMisc.h"
#include "Misc/FrameRate.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Misc/Parse.h"
#include "Misc/QualifiedFrameTime.h"
#include "Misc/Timecode.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

#include <atomic>
#include <optional>

class FCbObjectId;

/**
 * Provides information about the application.
 */
class FApp
{
public:

	/**
	 * Gets the name of the version control branch that this application was built from.
	 *
	 * @return The branch name.
	 */
	static CORE_API FString GetBranchName();

	/**
	 * Gets the application's build configuration, i.e. Debug or Shipping.
	 *
	 * @return The build configuration.
	 */
	static CORE_API EBuildConfiguration GetBuildConfiguration();

	/**
	 * Gets the target type of the current application (eg. client, server, etc...)
	 *
	 * @return The build target type
	 */
	static CORE_API EBuildTargetType GetBuildTargetType();

#if UE_BUILD_DEVELOPMENT
	/**
	 * For development configurations, sets whether the application should load DebugGame game modules.
	 *
	 * @param Whether we're running in debug game or not.
	 */
	static CORE_API void SetDebugGame(bool bIsDebugGame);
#endif

	/*
	* Gets the unique version string for this build. This string is not assumed to have any particular format other being a unique identifier for the build.
	*
	* @return The build version
	*/
	static CORE_API const TCHAR* GetBuildVersion();
	
	/* 
	 * Gets the URL for a job which created these binaries. 
	 * This may not be accessible if you are not part of the organization that created this build. 
	 * May be overridden with FCoreDelegates::OnGetBuildURL
	 */
	static CORE_API const TCHAR* GetBuildURL();

	/* 
	 * Gets the URL for the currently running Horde job/step if any. 
	 * May be overridden with FCoreDelegates::OnGetExecutingJobURL
	 */ 
	static CORE_API const TCHAR* GetExecutingJobURL();

	/* Returns whether the binaries were built with debug info */
	static CORE_API bool GetIsWithDebugInfo();

	/**
	 * Gets the date at which this application was built.
	 *
	 * @return Build date string.
	 */
	static CORE_API FString GetBuildDate();

	/**
	 * Gets the name of the graphics RHI currently in use.
	 *
	 * @return name of Graphics RHI
	 */
	static CORE_API FString GetGraphicsRHI();

	/**
	 * Sets the Graphics RHI currently in use
	 */
	static CORE_API void SetGraphicsRHI(FString RHIString);


	/**
	 * Gets the value of ENGINE_IS_PROMOTED_BUILD.
	 */
	static CORE_API int32 GetEngineIsPromotedBuild();

	/**
	 * Gets the identifier for the unreal engine
	 */
	static CORE_API FString GetEpicProductIdentifier();

	/**
	 * Gets the name of the current project.
	 *
	 * @return The project name.
	 */
	FORCEINLINE static const TCHAR* GetProjectName()
	{
		return GInternalProjectName;
	}

	/**
	 * Gets the name of the application, i.e. "UE" or "Rocket".
	 *
	 * @todo need better application name discovery. this is quite horrible and may not work on future platforms.
	 * @return Application name string.
	 */
	static FString GetName()
	{
		FString ExecutableName = FPlatformProcess::ExecutableName();

		int32 ChopIndex = ExecutableName.Find(TEXT("-"));

		if (ExecutableName.FindChar(TCHAR('-'), ChopIndex))
		{
			return ExecutableName.Left(ChopIndex);
		}

		if (ExecutableName.FindChar(TCHAR('.'), ChopIndex))
		{
			return ExecutableName.Left(ChopIndex);
		}

		return ExecutableName;
	}

	/**
	 * Reports if the project name has been set
	 *
	 * @return true if the project name has been set
	 */
	FORCEINLINE static bool HasProjectName()
	{
		return (IsProjectNameEmpty() == false) && (FCString::Stricmp(GInternalProjectName, TEXT("None")) != 0);
	}

	/**
	 * Checks whether this application is currently running as a game or other packaged application, and is not acting like an editor.
	 * In an editor build, this changes based on what the game thread is doing for Play In Editor so cannot be reliably accessed from worker threads.
	 *
	 * Returns true if a normal or PIE game is active, including a dedicated server.
	 * Use IsRunningGame if you want to check what the process was launched as and not the current state.
	 *
	 * @return true if the application is currently acting like a game, false otherwise.
	 */
	FORCEINLINE static bool IsGame()
	{
#if WITH_EDITOR
		return !GIsEditor || GIsPlayInEditorWorld;
#else
		return true;
#endif
	}

	/**
	 * Reports if the project name is empty
	 *
	 * @return true if the project name is empty
	 */
	FORCEINLINE static bool IsProjectNameEmpty()
	{
		return (GInternalProjectName[0] == 0);
	}

	/**
	* Sets the name of the current project.
	*
	* @param InProjectName Name of the current project.
	*/
	FORCEINLINE static void SetProjectName(const TCHAR* InProjectName)
	{
		// At the moment Strcpy is not safe as we don't check the buffer size on all platforms, so we use strncpy here.
		FCString::Strncpy(GInternalProjectName, InProjectName, UE_ARRAY_COUNT(GInternalProjectName));
		// And make sure the ProjectName string is null terminated.
		GInternalProjectName[UE_ARRAY_COUNT(GInternalProjectName) - 1] = TEXT('\0');
	}

	/**
	 * Add the specified user to the list of authorized session users.
	 *
	 * @param UserName The name of the user to add.
	 * @see DenyUser, DenyAllUsers, IsAuthorizedUser
	 */
	FORCEINLINE static void AuthorizeUser(const FString& UserName)
	{
		SessionUsers.AddUnique(UserName);
	}

	/**
	 * Removes all authorized users.
	 *
	 * @see AuthorizeUser, DenyUser, IsAuthorizedUser
	 */
	FORCEINLINE static void DenyAllUsers()
	{
		SessionUsers.Empty();
	}

	/**
	 * Remove the specified user from the list of authorized session users.
	 *
	 * @param UserName The name of the user to remove.
	 * @see AuthorizeUser, DenyAllUsers, IsAuthorizedUser
	 */
	FORCEINLINE static void DenyUser(const FString& UserName)
	{
		SessionUsers.Remove(UserName);
	}

	/**
	 * Gets the Zen store project id for the current project file path.
	 *
	 * @param SubProject An optional subproject identifier (used for DLC).
	 * @return Zen store project id.
	 */
	static CORE_API FString GetZenStoreProjectId(FStringView SubProject = FStringView());

#if PLATFORM_DESKTOP
	/**
	 * Gets the Zen store project id for an explicitly specified project file path.
	 *
	 * @param SubProject An optional subproject identifier (used for DLC).
	 * @return Zen store project id.
	 */
	static CORE_API FString GetZenStoreProjectIdForProject(FStringView ProjectFilePath, FStringView SubProject = FStringView());
#endif

	/**
	 * Gets the globally unique identifier of this application instance.
	 *
	 * Every running instance of the engine has a globally unique instance identifier
	 * that can be used to identify it from anywhere on the network.
	 *
	 * @return Instance identifier, or an invalid GUID if there is no local instance.
	 * @see GetSessionId
	 */
	static CORE_API FGuid GetInstanceId();

	/**
	 * Gets the name of this application instance.
	 *
	 * By default, the instance name is a combination of the computer name and process ID.
	 *
	 * @return Instance name string.
	 */
	static FString GetInstanceName()
	{
		return FString::Printf(TEXT("%s-%i"), FPlatformProcess::ComputerName(), FPlatformProcess::GetCurrentProcessId());
	}

	/**
	 * Gets the identifier of the session that this application is part of.
	 *
	 * A session is group of applications that were launched and logically belong together.
	 * For example, when starting a new session in UFE that launches a game on multiple devices,
	 * all engine instances running on those devices will have the same session identifier.
	 * Conversely, sessions that were launched separately will have different session identifiers.
	 *
	 * @return Session identifier, or an invalid GUID if there is no local instance.
	 * @see GetInstanceId
	 */
	FORCEINLINE static FGuid GetSessionId()
	{
		return SessionId;
	}

	/**
	 * Gets the identifier of the session that this application is part of as a FCbObject.
	 *
	 * @return Session identifier
	 * @see GetSessionId
	 */
	static CORE_API const FCbObjectId& GetSessionObjectId();

	/**
	 * Gets the name of the session that this application is part of, if any.
	 *
	 * @return Session name string.
	 */
	FORCEINLINE static FString GetSessionName()
	{
		return SessionName;
	}

	/**
	 * Gets the name of the user who owns the session that this application is part of, if any.
	 *
	 * If this application is part of a session that was launched from UFE, this function
	 * will return the name of the user that launched the session. If this application is
	 * not part of a session, this function will return the name of the local user account
	 * under which the application is running.
	 *
	 * @return Name of session owner.
	 */
	FORCEINLINE static FString GetSessionOwner()
	{
		return SessionOwner;
	}

	/**
	 * Initializes the application session.
	 */
	static CORE_API void InitializeSession();

	/**
	 * Check whether the specified user is authorized to interact with this session.
	 *
	 * @param UserName The name of the user to check.
	 * @return true if the user is authorized, false otherwise.
	 * @see AuthorizeUser, DenyUser, DenyAllUsers
	 */
	FORCEINLINE static bool IsAuthorizedUser(const FString& UserName)
	{
		return ((FPlatformProcess::UserName(false) == UserName) || (SessionOwner == UserName) || SessionUsers.Contains(UserName));
	}

	/**
	 * Checks whether this is a standalone application, where no session id is specified on the commandline.
	 *
	 * An application is standalone if it has not been launched as part of a session from UFE or another tool.
	 * This is not related to the networking or Play in Editor settings of the same name.
	 *
	 * @return true if this application is standalone, false otherwise.
	 */
	FORCEINLINE static bool IsStandalone()
	{
		return Standalone;
	}

	/**
	 * Check whether the given instance ID identifies this instance.
	 *
	 * @param InInstanceId The instance ID to check.
	 * @return true if the ID identifies this instance, false otherwise.
	 */
	FORCEINLINE static bool IsThisInstance(const FGuid& InInstanceId)
	{
		return (InInstanceId == GetInstanceId());
	};

	/**
	 * Set a new session name.
	 *
	 * @param NewName The new session name.
	 * @see SetSessionOwner
	 */
	FORCEINLINE static void SetSessionName(const FString& NewName)
	{
		SessionName = NewName;
	}

	/**
	 * Set a new session owner.
	 *
	 * @param NewOwner The name of the new owner.
	 * @see SetSessionName
	 */
	FORCEINLINE static void SetSessionOwner(const FString& NewOwner)
	{
		SessionOwner = NewOwner;
	}

	/**
	 * Checks whether this application can render anything.
	 * Certain application types never render, while for others this behavior may be controlled by switching to NullRHI.
	 * This can be used for decisions like omitting code paths that make no sense on servers or games running in headless mode (e.g. automated tests).
	 *
	 * @return true if the application can render, false otherwise.
	 */
	FORCEINLINE static bool CanEverRender()
	{
#if UE_SERVER
		return false;
#else
		static bool bHasNullRHIOnCommandline = FParse::Param(FCommandLine::Get(), TEXT("nullrhi"));
		return (!IsRunningCommandlet() || IsAllowCommandletRendering()) && !IsRunningDedicatedServer() && !(USE_NULL_RHI || bHasNullRHIOnCommandline);
#endif // UE_SERVER
	}

	/**
	 * Checks whether this application can render anything or produce a derived data needed for rednering.
	 * Certain application types never render, but produce DDC used during rendering and as such need to step into some rendering paths.
	 *
	 * A meaningful distinction from FApp::CanEverRender() is that commandlets like cooker will have FApp::CanEverRender() == false, but FApp::CanEverRenderOrProduceRenderData() == true.
	 * As such, this function can be used to guard paths that e.g. load assets' render data.
	 *
	 * @return true if the application can render, false otherwise.
	 */
	FORCEINLINE static bool CanEverRenderOrProduceRenderData()
	{
		return !FPlatformProperties::RequiresCookedData() || FApp::CanEverRender();
	}

	/**
	 * Checks whether this application can render audio.
	 * Certain application types produce sound, while for others this can be controlled via the -nosound cmdline.
	 * This can be used for decisions like omitting code paths that make no sense on servers or games running in headless mode (e.g. automated tests).
	 *
	 * @return true if the application can render audio, false otherwise.
	 */
	FORCEINLINE static bool CanEverRenderAudio()
	{
#if UE_SERVER
		return false;
#else
		static bool bHasNoAudioOnCommandline = FParse::Param(FCommandLine::Get(), TEXT("nosound")) && !FParse::Param(FCommandLine::Get(), TEXT("enablesound"));
		static bool bApplicationTypeDoesNotRenderAudio = FApp::IsBenchmarking() || IsRunningDedicatedServer() || (IsRunningCommandlet() && !IsAllowCommandletAudio());
		return !bApplicationTypeDoesNotRenderAudio && !bHasNoAudioOnCommandline;
#endif // UE_SERVER
	}

	/**
	 * Checks whether this application should mute the audio output (-muteaudio)
	 * This is a safer alternative to -nosound for users who want to mute the audio of the game but keep the game rendering sound. 
	 * Unless disabling the audio engine is the intention, this is a preferred method to mute the output as it avoid breaking audio analysis-dependent features (e.g. gameplay)
	 * and it prevents content creators from creating content with an incorrect performance and memory profile.
	 *
	 * The mute is applied only in non-shipping builds and applied at the final step in the final output submix.
	 * Alternative method to do this it to enable the cvar, "au.MuteAudio 1"
	 * 
	 * @return true if the application will mute audio, false otherwise.
	 */
	FORCEINLINE static bool IsAudioMuted()
	{
		static bool bMuteAudio = FParse::Param(FCommandLine::Get(), TEXT("muteaudio"));
		return bMuteAudio;
	}

	/**
	 * Checks whether this application has been installed.
	 *
	 * Non-server desktop shipping builds are assumed to be installed.
	 *
	 * Installed applications may not write to the folder in which they exist since they are likely in a system folder (like "Program Files" in windows).
	 * Instead, they should write to the OS-specific user folder FPlatformProcess::UserDir() or application data folder FPlatformProcess::ApplicationSettingsDir()
	 * User Access Control info for windows Vista and newer: http://en.wikipedia.org/wiki/User_Account_Control
	 *
	 * To run an "installed" build without installing, use the -Installed command line parameter.
	 * To run a "non-installed" desktop shipping build, use the -NotInstalled command line parameter.
	 *
	 * @return true if the application is installed, false otherwise.
	 */
	static CORE_API bool IsInstalled();

	/**
	 * Checks whether the engine components of this application have been installed.
	 *
	 * In binary Unreal Engine releases, the engine can be installed while the game is not. The game IsInstalled()
	 * setting will take precedence over this flag.
	 *
	 * To override, pass -engineinstalled or -enginenotinstalled on the command line.
	 *
	 * @return true if the engine is installed, false otherwise.
	 */
	static CORE_API bool IsEngineInstalled();

	/**
	 * Checks whether this application runs unattended.
	 *
	 * Unattended applications are not monitored by anyone or are unable to receive user input.
	 * This method can be used to determine whether UI pop-ups or other dialogs should be shown.
	 *
	 * @return true if the application runs unattended, false otherwise.
	 */
	static CORE_API bool IsUnattended();

	/**
	 * Checks whether this application allows unattended input.
	 *
	 * Unattended applications usually are unable to receive user input.
	 * This method can be used to determine whether to allow input modules to initialize when unattended.
	 *
	 * @return true if the application allows unattended input, false otherwise.
	 */
	static CORE_API bool AllowUnattendedInput();

	/**
	 * Checks whether the application should run multi-threaded for performance critical features.
	 *
	 * This method is used for performance based threads (like rendering, task graph).
	 * This will not disable async IO or similar threads needed to disable hitching
	 *
	 * @return true if this isn't a server, has more than one core, does not have a -onethread command line options, etc.
	 */
	static CORE_API bool ShouldUseThreadingForPerformance();

	/**
	 * Returns if the application is a dedicated server and wants to be multi-threaded.
	 */
	static CORE_API bool IsMultithreadServer();

	/**
	 * Checks whether application is in benchmark mode.
	 *
	 * @return true if application is in benchmark mode, false otherwise.
	 */
	FORCEINLINE static bool IsBenchmarking()
	{
		return bIsBenchmarking;
	}

	/**
	 * Sets application benchmarking mode.
	 *
	 * @param bVal True sets application in benchmark mode, false sets to non-benchmark mode.
	 */
	static void SetBenchmarking(bool bVal)
	{
		bIsBenchmarking = bVal;
	}

	/**
	 * Gets time step in seconds if a fixed delta time is wanted.
	 *
	 * @return Time step in seconds for fixed delta time.
	 */
	static double GetFixedDeltaTime()
	{
		return FAppTime::GetFixedDeltaTime();
	}
	
	/**
	 * Gets whether we want to use a fixed time step or not.
	 *
	 * @return True if using fixed time step, false otherwise.
	 */
	static bool UseFixedTimeStep()
	{
		return FAppTime::UseFixedTimeStep();
	}

	/**
	 * Gets current time in seconds.
	 *
	 * @return Current time in seconds.
	 */
	static double GetCurrentTime()
	{
		return FAppTime::GetCurrentTime();
	}

	/**
	 * Gets previous value of CurrentTime.
	 *
	 * @return Previous value of CurrentTime.
	 */
	static double GetLastTime()
	{
		return FAppTime::GetLastTime();
	}

	/**
	 * Gets time delta in seconds.
	 *
	 * @return Time delta in seconds.
	 */
	static double GetDeltaTime()
	{
		return FAppTime::GetDeltaTime();
	}

	/**
	 * Gets idle time in seconds.
	 *
	 * @return Idle time in seconds.
	 */
	static double GetIdleTime()
	{
		return FAppTime::GetIdleTime();
	}

	/**
	 * Gets overall game time in seconds.
	 *
	 * @return Overall game time in seconds.
	 */
	static double GetGameTime()
	{
		return FAppTime::GetGameTime();
	}

	/**
	 * Gets idle time overshoot in seconds (the time beyond the wait time we requested for the frame). Only valid when IdleTime is > 0.
	 *
	 * @return Idle time in seconds.
	 */
	static double GetIdleTimeOvershoot()
	{
		return FAppTime::GetIdleTimeOvershoot();
	}

	/**
	 * Convert the current frame time into a readable timecode.
	 * If the current frame time is not set, the timecode value will be defaulted.
	 *
	 * @return the current timecode.
	 */
	static FTimecode GetTimecode()
	{
		return FAppTime::GetTimecode();
	}

	/**
	 * Get the frame rate of the current frame time.
	 * If the current frame time is not set, the frame rate value will be defaulted.
	 *
	 * @return the current timecode frame rate.
	 */
	static FFrameRate GetTimecodeFrameRate()
	{
		return FAppTime::GetTimecodeFrameRate();
	}

	/**
	 * Gets a frame number generated by the engine's timecode provider.
	 * The current frame time is used to generate a timecode value.
	 * The optional will be false if no timecode provider was set or is not in a synchronized state.
	 *
	 * @return the current frame time.
	 */
	static TOptional<FQualifiedFrameTime> GetCurrentFrameTime()
	{
		return FAppTime::GetCurrentFrameTime();
	}

	/**
	 * Sets time step in seconds if a fixed delta time is wanted.
	 *
	 * @param seconds Time step in seconds for fixed delta time.
	 */
	// UE_DEPRECATED(5.8, "@todo delta time refactor - deprecate this function")
	static void SetFixedDeltaTime(double Seconds)
	{
		FAppTime::SetFixedDeltaTime(Seconds);
	}

	/**
	 * Enables or disabled usage of fixed time step.
	 *
	 * @param whether to use fixed time step or not
	 */
	// UE_DEPRECATED(5.8, "@todo delta time refactor - deprecate this function")
	static void SetUseFixedTimeStep(bool bVal)
	{
		FAppTime::SetUseFixedTimeStep(bVal);
	}

	/**
	 * Sets current time in seconds.
	 *
	 * @param seconds - Time in seconds.
	 */
	// UE_DEPRECATED(5.8, "@todo delta time refactor - deprecate this function")
	static void SetCurrentTime(double Seconds)
	{
		FAppTime::SetCurrentTime(Seconds);
	}

	/** Updates Last time to CurrentTime. */
	// UE_DEPRECATED(5.8, "@todo delta time refactor - deprecate this function")
	static void UpdateLastTime()
	{
		FAppTime::UpdateLastTime();
	}

	/**
	 * Sets time delta in seconds.
	 *
	 * @param seconds Time in seconds.
	 */
	// UE_DEPRECATED(5.8, "@todo delta time refactor - deprecate this function")
	static void SetDeltaTime(double Seconds)
	{
		FAppTime::SetDeltaTime(Seconds);
	}

	/**
	 * Sets idle time in seconds.
	 *
	 * @param seconds - Idle time in seconds.
	 */
	// UE_DEPRECATED(5.8, "@todo delta time refactor - deprecate this function")
	static void SetIdleTime(double Seconds)
	{
		FAppTime::SetIdleTime(Seconds);
	}

	/**
	 * Sets overall game time in seconds.
	 *
	 * @param seconds - Overall game time in seconds.
	 */
	// UE_DEPRECATED(5.8, "@todo delta time refactor - deprecate this function")
	static void SetGameTime(double Seconds)
	{
		FAppTime::SetGameTime(Seconds);
	}

	/**
	 * Sets idle time overshoot in seconds (the time beyond the wait time we requested for the frame). Only valid when IdleTime is > 0.
	 *
	 * @param seconds - Idle time in seconds.
	 */
	// UE_DEPRECATED(5.8, "@todo delta time refactor - deprecate this function")
	static void SetIdleTimeOvershoot(double Seconds)
	{
		FAppTime::SetIdleTimeOvershoot(Seconds);
	}

	/**
	 * Sets the current frame time.
	 *
	 * @param InFrameTime - current frame time and frame rate.
	 */
	// UE_DEPRECATED(5.8, "@todo delta time refactor - deprecate this function")
	static void SetCurrentFrameTime(FQualifiedFrameTime InFrameTime)
	{
		FAppTime::SetCurrentFrameTime(InFrameTime);
	}

	/** Invalidate the current frame time. It will reset the TOptional. */
	// UE_DEPRECATED(5.8, "@todo delta time refactor - deprecate this function")
	static void InvalidateCurrentFrameTime()
	{
		FAppTime::InvalidateCurrentFrameTime();
	}

	/**
	 * Get Volume Multiplier
	 * 
	 * @return Current volume multiplier
	 */
	FORCEINLINE static float GetVolumeMultiplier()
	{
		return VolumeMultiplier;
	}

	/**
	 * Set Volume Multiplier
	 * 
	 * @param  InVolumeMultiplier	new volume multiplier
	 */
	FORCEINLINE static void SetVolumeMultiplier(float InVolumeMultiplier)
	{
		VolumeMultiplier = InVolumeMultiplier;
	}

	/**
	 * Helper function to get UnfocusedVolumeMultiplier from config and store so it's not retrieved every frame
	 * 
	 * @return Volume multiplier to use when app loses focus
	 */
	static CORE_API float GetUnfocusedVolumeMultiplier();

	/**
	* Sets the Unfocused Volume Multiplier
	*/
	static CORE_API void SetUnfocusedVolumeMultiplier(float InVolumeMultiplier);

	/**
	 * Sets if VRFocus should be used.
	 *
	 * @param  bInUseVRFocus	new bUseVRFocus value
	 */
	static CORE_API void SetUseVRFocus(bool bInUseVRFocus);

	/**
	 * Gets if VRFocus should be used
	 */
	FORCEINLINE static bool UseVRFocus()
	{
		return bUseVRFocus;
	}
	/**
	 * Sets VRFocus, which indicates that the application should continue to render 
	 * Audio and Video as if it had window focus, even though it may not.
	 *
	 * @param  bInVRFocus	new VRFocus value
	 */
	static CORE_API void SetHasVRFocus(bool bInHasVRFocus);

	/**
	 * Gets VRFocus, which indicates that the application should continue to render 
	 * Audio and Video as if it had window focus, even though it may not.
	 */
	FORCEINLINE static bool HasVRFocus()
	{
		return bHasVRFocus;
	}
	
	/**
	 * Sets HasFocus, which is a function that indicates that the application window has focus.
	 *
	 * @param  InHasFocusFunction	address of a function that can query the focus state, typically &FPlatformApplicationMisc::IsThisApplicationForeground
	 */
	static CORE_API void SetHasFocusFunction(bool (*InHasFocusFunction)());

	/**
	 * Gets Focus, Indicates that the application should continue to render
	 * Audio and Video as if it had window focus, even though it may not.
	 */
	static CORE_API bool HasFocus();

	/* If the random seed started with a constant or on time, can be affected by -FIXEDSEED or -BENCHMARK */
	static CORE_API bool bUseFixedSeed;

	/* Print all initial startup logging */
	static CORE_API void PrintStartupLogMessages();

private:

#if UE_BUILD_DEVELOPMENT
	/** The current build configuration */
	static CORE_API bool bIsDebugGame;
#endif

	/** Holds the instance identifier. */
	static CORE_API FGuid InstanceId;

	/** Holds the session identifier. */
	static CORE_API FGuid SessionId;

	/** Holds the session name. */
	static CORE_API FString SessionName;

	/** Holds the name of the user that launched session. */
	static CORE_API FString SessionOwner;

	/** Holds the name the graphics RHI currently in use*/
	static CORE_API FString GraphicsRHI;

	/** List of authorized session users. */
	static CORE_API TArray<FString> SessionUsers;

	/** Holds a flag indicating whether this is a standalone session. */
	static CORE_API bool Standalone;

	/** Holds a flag Whether we are in benchmark mode or not. */
	static CORE_API bool bIsBenchmarking;

	/** Use to affect the app volume when it loses focus */
	static CORE_API float VolumeMultiplier;

	/** Read from config to define the volume when app loses focus */
	static CORE_API float UnfocusedVolumeMultiplier;

	/** Holds a flag indicating if VRFocus should be used */
	static CORE_API bool bUseVRFocus;

	/** Holds a flag indicating if app has focus in side the VR headset */
	static CORE_API bool bHasVRFocus;

	/** Holds a function address that can indicate if application has focus */
	static CORE_API bool (*HasFocusFunction)();
};


/** Called to determine the result of IsServerForOnlineSubsystems() */
DECLARE_DELEGATE_RetVal_OneParam(bool, FQueryIsRunningServer, FName);


/**
 * @return true if there is a running game world that is a server (including listen servers), false otherwise
 */
CORE_API bool IsServerForOnlineSubsystems(FName WorldContextHandle);

/**
 * Sets the delegate used for IsServerForOnlineSubsystems
 */
CORE_API void SetIsServerForOnlineSubsystemsDelegate(FQueryIsRunningServer NewDelegate);
