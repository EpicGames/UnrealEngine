// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "Misc/Optional.h"
#include "Misc/QualifiedFrameTime.h"
#include "Misc/Timecode.h"
#include "Templates/SharedPointer.h"

namespace UE
{
	class FInheritedContextScope;
	class FInheritedContextBase;
}

/**
 * Provides global game timing information (delta time, current time etc).
 * The current time is set each frame by the game thread. It may then be retrieved by the game thread and other task threads via FAppTime::Get().
 * The value from the game thread is pipelined through the TaskGraph such that child tasks receive the correct value for the frame the tasks were spawned in.
 */
class FAppTime
{
public:
	// Helper type to perform automatic relaxed-order loads/stores of a std::atomic.
	template <typename TValue>
	struct TRelaxedAtomic
	{
		std::atomic<TValue> Value;

		TRelaxedAtomic() = default;

		TRelaxedAtomic(TRelaxedAtomic const& Other)
		{
			(*this) = Other.Value;
		}

		TRelaxedAtomic(TValue InValue)
		{
			(*this) = InValue;
		}

		TRelaxedAtomic& operator = (TValue InValue)
		{
			Value.store(InValue, std::memory_order_relaxed);
			return *this;
		}

		TRelaxedAtomic& operator = (TRelaxedAtomic const& RHS)
		{
			TValue LoadedValue = RHS;
			(*this) = LoadedValue;
			return *this;
		}

		operator TValue() const
		{
			return Value.load(std::memory_order_relaxed);
		}
	};

	friend UE::FInheritedContextScope;
	friend UE::FInheritedContextBase;
	friend class FApp;

#if WITH_FIXED_TIME_STEP_SUPPORT
	// Holds a flag whether we want to use a fixed time step or not.
	bool bUseFixedTimeStep = false;
#else
	static constexpr bool bUseFixedTimeStep = false;
#endif

	// Holds time step if a fixed delta time is wanted.
	double FixedDeltaTime = 1 / 30.0;

	// Holds current time.
	TRelaxedAtomic<double> CurrentTime = 0.0;

	// Holds previous value of CurrentTime.
	double LastTime = 0.0;

	// Holds current delta time in seconds.
	double DeltaTime = 1 / 30.0;

	// Holds time we spent sleeping in UpdateTimeAndHandleMaxTickRate() if our frame time was smaller than one allowed by target FPS.
	double IdleTime = 0.0;

	// Holds the amount of IdleTime that was LONGER than we tried to sleep. The OS can't sleep the exact amount of time, so this measures that overshoot.
	double IdleTimeOvershoot = 0.0;

	// Holds overall game time.
	double GameTime = 0.0;

	/** Holds the current frame time and framerate. */
	TOptional<FQualifiedFrameTime> CurrentFrameTime;

private:
	//
	// Returns the FAppTime value for the current thread's time context. May be called by the game thread, or decendent TaskGraph tasks of the game thread.
	// Calling this function from any other context will result in an ensure failure. This is because the game thread is the authority on the FAppTime value,
	// and this value is pipelined through the TaskGraph so that decendent tasks receive the correct time value for the frame they belong to.
	// Reading the FAppTime value outside of these contexts is a data race with the game thread.
	//
	static CORE_API const FAppTime& Get();

public:
	UE_DEPRECATED(5.8,
		"Do not call FAppTime::Get_UnsafeNoCheck() in new engine code. This function is provided "
		"as a temporary measure to avoid triggering the time context ensure in FAppTime::Get() "
		"for engine systems that read the FAppTime value in a non-threadsafe manner, until those "
		"systems can be refactored.")
	static CORE_API const FAppTime& Get_UnsafeNoCheck();

	// Sets the new FAppTime value. May only be called from the game thread.
	static CORE_API void Set(const FAppTime& NewValue);

	//
	// Setter functions matching those from FApp
	//

	static void SetFixedDeltaTime(double Seconds)
	{
		FAppTime Time = Get();
		Time.FixedDeltaTime = Seconds;
		Set(Time);
	}

	static void SetUseFixedTimeStep(bool bVal)
	{
#if WITH_FIXED_TIME_STEP_SUPPORT
		FAppTime Time = Get();
		Time.bUseFixedTimeStep = bVal;
		Set(Time);
#endif
	}

	static void SetCurrentTime(double Seconds)
	{
		FAppTime Time = Get();
		Time.CurrentTime = Seconds;
		Set(Time);
	}

	static void UpdateLastTime()
	{
		FAppTime Time = Get();
		Time.LastTime = Time.CurrentTime;
		Set(Time);
	}

	static void SetDeltaTime(double Seconds)
	{
		FAppTime Time = Get();
		Time.DeltaTime = Seconds;
		Set(Time);
	}

	static void SetIdleTime(double Seconds)
	{
		FAppTime Time = Get();
		Time.IdleTime = Seconds;
		Set(Time);
	}

	static void SetGameTime(double Seconds)
	{
		FAppTime Time = Get();
		Time.GameTime = Seconds;
		Set(Time);
	}

	static void SetIdleTimeOvershoot(double Seconds)
	{
		FAppTime Time = Get();
		Time.IdleTimeOvershoot = Seconds;
		Set(Time);
	}

	static void SetCurrentFrameTime(FQualifiedFrameTime InFrameTime)
	{
		FAppTime Time = Get();
		Time.CurrentFrameTime = InFrameTime;
		Set(Time);
	}

	static void InvalidateCurrentFrameTime()
	{
		FAppTime Time = Get();
		Time.CurrentFrameTime.Reset();
		Set(Time);
	}

	//
	// Equivalent to GetCurrentTime(), but can be called from any thread, rather than only from threads with valid time contexts.
	// The returned value is loaded via a relaxed atomic read, and therefore can race updates from the game thread.
	// 
	// Use only for systems that require an approximate incrementing timer, where getting a value from a frame that is off-by-one is not important.
	// Do not use in systems where the current time directly affects the game simulation or rendered pixels. Those systems should use GetCurrentTime()
	// to ensure they retrieve the appropriate time value via the inherited time context.
	//
	static double GetApproximateCurrentTime() { return GameThreadInstance.CurrentTime; }

	//
	// Pipelined Getter functions.
	// Must only be called from the game thread, or threads with a valid time context.
	//

	// Gets time step in seconds if a fixed delta time is wanted.
	static double GetFixedDeltaTime() { return Get().FixedDeltaTime; }

	// Gets whether we want to use a fixed time step or not.
	static bool UseFixedTimeStep() { return Get().bUseFixedTimeStep; }

	// Gets current frame's time in seconds.
	static double GetCurrentTime() { return Get().CurrentTime; }

	// Gets the previous frame's value of CurrentTime
	static double GetLastTime() { return Get().LastTime; }

	// Gets the current frame's time delta in seconds.
	static double GetDeltaTime() { return Get().DeltaTime; }

	// Gets idle time in seconds.
	static double GetIdleTime() { return Get().IdleTime; }

	// Gets overall game time in seconds.
	static double GetGameTime() { return Get().GameTime; }

	// Gets idle time overshoot in seconds (the time beyond the wait time we requested for the frame). Only valid when IdleTime is > 0.
	static double GetIdleTimeOvershoot() { return Get().IdleTimeOvershoot; }

	//
	// Gets a frame number generated by the engine's timecode provider.
	// The current frame time is used to generate a timecode value.
	// The optional will be false if no timecode provider was set or is not in a synchronized state.
	//
	static TOptional<FQualifiedFrameTime> GetCurrentFrameTime()
	{
		return Get().CurrentFrameTime;
	}

	//
	// Convert the current frame time into a readable timecode.
	// If the current frame time is not set, the timecode value will be defaulted.
	//
	static FTimecode GetTimecode()
	{
		TOptional<FQualifiedFrameTime> CurrentFrameTime = GetCurrentFrameTime();

		if (CurrentFrameTime.IsSet())
		{
			return CurrentFrameTime->ToTimecode();
		}

		return FTimecode();
	}

	// Get the frame rate of the current frame time. If the current frame time is not set, the frame rate value will be defaulted.
	static FFrameRate GetTimecodeFrameRate()
	{
		TOptional<FQualifiedFrameTime> CurrentFrameTime = GetCurrentFrameTime();

		if (CurrentFrameTime.IsSet())
		{
			return CurrentFrameTime->Rate;
		}

		return FFrameRate();
	}

private:
	static CORE_API FAppTime GameThreadInstance;
	static thread_local TSharedPtr<const FAppTime> TLSInstance;

	[[nodiscard]] static CORE_API TSharedPtr<const FAppTime> Fork();
	static CORE_API void Restore(TSharedPtr<const FAppTime>&& Ptr);
};
