// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/SecureHash.h"
#include "Serialization/Archive.h"
#include "UObject/NameTypes.h"

/**
 * FArchive adapter for FMD5
 */
class FArchiveMD5 : public FArchive
{
public:
	inline FArchiveMD5()
	{
		SetIsLoading(false);
		SetIsSaving(true);
		SetIsPersistent(false);
	}

	CORE_API virtual FString GetArchiveName() const;

	void Serialize(void* Data, int64 Num) override
	{
		MD5.Update((uint8*)Data, Num);
	}

	using FArchive::operator<<;

	virtual FArchive& operator<<(class FName& Value) override
	{
		FString NameAsString = Value.ToString();
		*this << NameAsString;
		return *this;
	}

	virtual FArchive& operator<<(class UObject*& Value) override
	{
		check(0);
		return *this;
	}

	virtual void Reset() override
	{
		FArchive::Reset();
		MD5 = FMD5();
	}

	void GetHash(FMD5Hash& Hash)
	{
		Hash.Set(MD5);
	}

	FGuid GetGuidFromHash()
	{
		FMD5Hash MD5Hash;
		GetHash(MD5Hash);
		return MD5HashToGuid(MD5Hash);
	}

protected:
	FMD5 MD5;
};
