// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Char.h"

/**
 * ASCII character bitset useful for fast and readable parsing
 *
 * Entirely constexpr. Works with both wide and narrow strings.
 *
 * Example use cases:
 *
 *   constexpr FAsciiSet WhitespaceCharacters(" \v\f\t\r\n");
 *   bool bIsWhitespace = WhitespaceCharacters.Contains(MyChar);
 *   const char* HelloWorld = FAsciiSet::Skip("  \t\tHello world!", WhitespaceCharacters);
 *   
 *   constexpr FAsciiSet XmlEscapeChars("&<>\"'");
 *   check(FAsciiSet::HasNone(EscapedXmlString, XmlEscapeChars));
 *   
 *   constexpr FAsciiSet Delimiters(".:;");
 *   const TCHAR* DelimiterOrEnd = FAsciiSet::FindFirstOrEnd(PrefixedName, Delimiters);
 *   FString Prefix(PrefixedName, DelimiterOrEnd - PrefixedName);
 *   
 *   constexpr FAsciiSet Slashes("/\\");
 *   const TCHAR* SlashOrEnd = FAsciiSet::FindLastOrEnd(PathName, Slashes);
 *   const TCHAR* FileName = *SlashOrEnd ? SlashOrEnd + 1 : PathName;
 */
class FAsciiSet
{
public:
	template<typename CharType, int N>
	constexpr FAsciiSet(const CharType(&Chars)[N])
		: FAsciiSet(StringToBitset(Chars))
	{}

	/** Returns true if a character is part of the set */
	template<typename CharType>
	constexpr FORCEINLINE bool Contains(CharType Char) const
	{
		return !!TestImpl(TChar<CharType>::ToUnsigned(Char));
	}

	/** Returns non-zero if a character is part of the set. Prefer Contains() to avoid VS2019 conversion warnings. */
	template<typename CharType>
	constexpr FORCEINLINE uint64 Test(CharType Char) const
	{
		return TestImpl(TChar<CharType>::ToUnsigned(Char));
	}

	/** Create new set with specified character in it */
	constexpr FORCEINLINE FAsciiSet operator+(char Char) const
	{
		InitData Bitset = { LoMask, HiMask };
		SetImpl(Bitset, TChar<char>::ToUnsigned(Char));
		return FAsciiSet(Bitset);
	}

	/** Create new set containing the combination of this set and another */
	constexpr FORCEINLINE FAsciiSet operator|(FAsciiSet OtherSet) const
	{
		return FAsciiSet(LoMask | OtherSet.LoMask, HiMask | OtherSet.HiMask);
	}

	/** Create new set containing only the characters that exist in both this and another specified set */
	constexpr FORCEINLINE FAsciiSet operator&(FAsciiSet OtherSet) const
	{
		return FAsciiSet(LoMask & OtherSet.LoMask, HiMask & OtherSet.HiMask);
	}

	/** Create new set containing inverse set of characters - likely including null-terminator */
	constexpr FORCEINLINE FAsciiSet operator~() const
	{
		return FAsciiSet(~LoMask, ~HiMask);
	}

	////////// Algorithms for C strings //////////

	/** Find first character of string inside set or end pointer. Never returns null. */
	template<class CharType>
	static constexpr const CharType* FindFirstOrEnd(const CharType* Str, FAsciiSet Set)
	{
		for (FAsciiSet SetOrNil(Set.LoMask | NilMask, Set.HiMask); !SetOrNil.Test(*Str); ++Str);

		return Str;
	}

	/** Find last character of string inside set or end pointer. Never returns null. */
	template<class CharType>
	static constexpr const CharType* FindLastOrEnd(const CharType* Str, FAsciiSet Set)
	{
		const CharType* Last = FindFirstOrEnd(Str, Set);

		for (const CharType* It = Last; *It; It = FindFirstOrEnd(It + 1, Set))
		{
			Last = It;
		}

		return Last;
	}

	/** Find first character of string outside of set. Never returns null. */
	template<typename CharType>
	static constexpr const CharType* Skip(const CharType* Str, FAsciiSet Set)
	{
		while (Set.Contains(*Str))
		{
			++Str;
		}
        
        return Str;
	}

	/** Test if string contains any character in set */
	template<typename CharType>
	static constexpr bool HasAny(const CharType* Str, FAsciiSet Set)
	{
		return *FindFirstOrEnd(Str, Set) != '\0';
	}

	/** Test if string contains no character in set */
	template<typename CharType>
	static constexpr bool HasNone(const CharType* Str, FAsciiSet Set)
	{
		return *FindFirstOrEnd(Str, Set) == '\0';
	}

	/** Test if string contains any character outside of set */
	template<typename CharType>
	static constexpr bool HasOnly(const CharType* Str, FAsciiSet Set)
	{
		return *Skip(Str, Set) == '\0';
	}

	////////// Algorithms for string types like FStringView and FString //////////

	/** Get initial substring with all characters in set */
	template<class StringType>
	static constexpr StringType FindPrefixWith(const StringType& Str, FAsciiSet Set)
	{
		return Scan<EDir::Forward, EInclude::Members, EKeep::Head>(Str, Set);
	}

	/** Get initial substring with no characters in set */
	template<class StringType>
	static constexpr StringType FindPrefixWithout(const StringType& Str, FAsciiSet Set)
	{
		return Scan<EDir::Forward, EInclude::NonMembers, EKeep::Head>(Str, Set);
	}

	/** Trim initial characters in set */
	template<class StringType>
	static constexpr StringType TrimPrefixWith(const StringType& Str, FAsciiSet Set)
	{
		return Scan<EDir::Forward, EInclude::Members, EKeep::Tail>(Str, Set);
	}

	/** Trim initial characters not in set */
	template<class StringType>
	static constexpr StringType TrimPrefixWithout(const StringType& Str, FAsciiSet Set)
	{
		return Scan<EDir::Forward, EInclude::NonMembers, EKeep::Tail>(Str, Set);
	}

	/** Get trailing substring with all characters in set */
	template<class StringType>
	static constexpr StringType FindSuffixWith(const StringType& Str, FAsciiSet Set)
	{
		return Scan<EDir::Reverse, EInclude::Members, EKeep::Tail>(Str, Set);
	}

	/** Get trailing substring with no characters in set */
	template<class StringType>
	static constexpr StringType FindSuffixWithout(const StringType& Str, FAsciiSet Set)
	{
		return Scan<EDir::Reverse, EInclude::NonMembers, EKeep::Tail>(Str, Set);
	}

	/** Trim trailing characters in set */
	template<class StringType>
	static constexpr StringType TrimSuffixWith(const StringType& Str, FAsciiSet Set)
	{
		return Scan<EDir::Reverse, EInclude::Members, EKeep::Head>(Str, Set);
	}

	/** Trim trailing characters not in set */
	template<class StringType>
	static constexpr StringType TrimSuffixWithout(const StringType& Str, FAsciiSet Set)
	{
		return Scan<EDir::Reverse, EInclude::NonMembers, EKeep::Head>(Str, Set);
	}
	
	/** Test if string contains any character in set */
	template<class StringType>
	static constexpr bool HasAny(const StringType& Str, FAsciiSet Set)
	{
		return !HasNone(Str, Set);
	}

	/** Test if string contains no character in set */
	template<class StringType>
	static constexpr bool HasNone(const StringType& Str, FAsciiSet Set)
	{
		uint64 Match = 0;
		for (auto Char : Str)
		{
			Match |= Set.Test(Char);
		}
		return Match == 0;
	}

	/** Test if string contains any character outside of set */
	template<class StringType>
	static constexpr bool HasOnly(const StringType& Str, FAsciiSet Set)
	{
		auto End = GetData(Str) + GetNum(Str);
		return FindFirst<EInclude::Members>(Set, GetData(Str), End) == End;
	}

private:
	enum class EDir {Forward, Reverse};
	enum class EInclude {Members, NonMembers};
	enum class EKeep {Head, Tail};

	template<EInclude Include, typename CharType>
	static constexpr const CharType* FindFirst(FAsciiSet Set, const CharType* It, const CharType* End)
	{
		for (; It != End && (Include == EInclude::Members) == !!Set.Test(*It); ++It);
		return It;
	}

	template<EInclude Include, typename CharType>
	static constexpr const CharType* FindLast(FAsciiSet Set, const CharType* It, const CharType* End)
	{
		for (; It != End && (Include == EInclude::Members) == !!Set.Test(*It); --It);
		return It;
	}

	template<EDir Dir, EInclude Include, EKeep Keep, class StringType>
	static constexpr StringType Scan(const StringType& Str, FAsciiSet Set)
	{
		auto Begin = GetData(Str);
		auto End = Begin + GetNum(Str);
		auto It = Dir == EDir::Forward	? FindFirst<Include>(Set, Begin, End)
										: FindLast<Include>(Set, End - 1, Begin - 1) + 1;

		return Keep == EKeep::Head	? StringType(Begin, static_cast<int32>(It - Begin))
									: StringType(It, static_cast<int32>(End - It));
	}

	// Work-around for constexpr limitations
	struct InitData { uint64 Lo, Hi; };
	static constexpr uint64 NilMask = uint64(1) << '\0';

	static constexpr FORCEINLINE void SetImpl(InitData& Bitset, uint32 Char)
	{
		uint64 IsLo = uint64(0) - (Char >> 6 == 0);
		uint64 IsHi = uint64(0) - (Char >> 6 == 1);
		uint64 Bit = uint64(1) << uint8(Char & 0x3f);

		Bitset.Lo |= Bit & IsLo;
		Bitset.Hi |= Bit & IsHi;
	}

	constexpr FORCEINLINE uint64 TestImpl(uint32 Char) const
	{
		uint64 IsLo = uint64(0) - (Char >> 6 == 0);
		uint64 IsHi = uint64(0) - (Char >> 6 == 1);
		uint64 Bit = uint64(1) << (Char & 0x3f);

		return (Bit & IsLo & LoMask) | (Bit & IsHi & HiMask);
	}

	template<typename CharType, int N>
	static constexpr InitData StringToBitset(const CharType(&Chars)[N])
	{
		InitData Bitset = { 0, 0 };
		for (int I = 0; I < N - 1; ++I)
		{
			SetImpl(Bitset, TChar<CharType>::ToUnsigned(Chars[I]));
		}

		return Bitset;
	}

	constexpr FAsciiSet(InitData Bitset) 
		: LoMask(Bitset.Lo), HiMask(Bitset.Hi)
	{}

	constexpr FAsciiSet(uint64 Lo, uint64 Hi)
		: LoMask(Lo), HiMask(Hi)
	{}

	uint64 LoMask, HiMask;
};
