// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PreprocessorHelpers.h"
#include "Misc/VarArgs.h"
#include "String/FormatStringSan.h"
#include "Templates/EnableIf.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Templates/IsValidVariadicFunctionArg.h"
#include "Traits/IsCharEncodingCompatibleWith.h"

#include <atomic>

#ifndef UE_DEBUG_SECTION
#if (DO_CHECK || DO_GUARD_SLOW || DO_ENSURE) && !PLATFORM_CPU_ARM_FAMILY
	// We'll put all assert implementation code into a separate section in the linked
	// executable. This code should never execute so using a separate section keeps
	// it well off the hot path and hopefully out of the instruction cache. It also
	// facilitates reasoning about the makeup of a compiled/linked binary.
	// Also see UE_COLD.
	#define UE_DEBUG_SECTION PLATFORM_CODE_SECTION(".uedbg")
#else
	// On ARM we can't do this because the executable will require jumps larger
	// than the branch instruction can handle. Clang will only generate
	// the trampolines in the .text segment of the binary. If the uedbg segment
	// is present it will generate code that it cannot link.
	// Consider using UE_COLD instead.
	#define UE_DEBUG_SECTION
#endif // DO_CHECK || DO_GUARD_SLOW
#endif

namespace ELogVerbosity
{
	enum Type : uint8;
}
/**
 * C Exposed function to print the callstack to ease debugging needs.  In an 
 * editor build you can call this in the Immediate Window by doing, {,,UnrealEditor-Core}::PrintScriptCallstack()
 */
extern "C" CORE_API void PrintScriptCallstack();


struct FEnsureHandlerArgs
{
	const ANSICHAR* Expression;
	const TCHAR* Message;
};

template <typename FuncType> class TFunction;

/** 
 * Sets the thead_local ensure handler callback
 * returning true skips the regular ensure handling logic
 * returns the previous handler which maybe nullptr
 **/
CORE_API TFunction<bool(const FEnsureHandlerArgs& Args)> SetEnsureHandler(TFunction<bool(const FEnsureHandlerArgs& Args)> EnsureHandler);
CORE_API TFunction<bool(const FEnsureHandlerArgs& Args)> GetEnsureHandler();

/**
 * FDebug
 * These functions offer debugging and diagnostic functionality and its presence 
 * depends on compiler switches.
 **/
struct FDebug
{
	/** Logs final assert message and exits the program. */
	static CORE_API void VARARGS AssertFailed(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, const TCHAR* Format = TEXT(""), ...);
	static CORE_API void AssertFailedV(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, const TCHAR* Format, va_list Args);

	/** Triggers a fatal error, using the error formatted to GErrorHist via a previous call to FMsg*/
	static CORE_API void ProcessFatalError(void* ProgramCounter);

	// returns true if an assert has occurred
	static CORE_API bool HasAsserted();

	// returns true if an ensure is currently in progress (e.g. the RenderThread is ensuring)
	static CORE_API bool IsEnsuring();

	// returns the number of times an ensure has failed in this instance.
	static CORE_API SIZE_T GetNumEnsureFailures();

	/** Dumps the stack trace into the log, meant to be used for debugging purposes. */
	static CORE_API void DumpStackTraceToLog(const ELogVerbosity::Type LogVerbosity);

	/** Dumps the stack trace into the log with a custom heading, meant to be used for debugging purposes. */
	static CORE_API void DumpStackTraceToLog(const TCHAR* Heading, const ELogVerbosity::Type LogVerbosity);

#if DO_CHECK || DO_GUARD_SLOW || DO_ENSURE
public:
	static CORE_API bool VARARGS CheckVerifyFailedImpl(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* Format, ...);
	static CORE_API bool VARARGS CheckVerifyFailedImpl2(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, const TCHAR* Format, ...);
private:
	static CORE_API void VARARGS LogAssertFailedMessageImpl(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* Fmt, ...);
	static CORE_API void LogAssertFailedMessageImplV(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* Fmt, va_list Args);
	static CORE_API bool CheckVerifyFailedImpl2V(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, const TCHAR* Format, va_list Args);

public:
//	/**
//	 * Called when a 'check/verify' assertion fails.
//	 */
//	template <typename FmtType, typename... Types>
//	static void UE_DEBUG_SECTION CheckVerifyFailed(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, void* ProgramCounter, const FmtType& Format, Types... Args);
	
	/**
	 * Called when an 'ensure' assertion fails; gathers stack data and generates and error report.
	 *
	 * @param	Expr	Code expression ANSI string (#code)
	 * @param	File	File name ANSI string (__FILE__)
	 * @param	Line	Line number (__LINE__)
	 * @param	Msg		Informative error message text
	 * @param	NumStackFramesToIgnore	Number of stack frames to ignore in the ensure message
	 * 
	 * Don't change the name of this function, it's used to detect ensures by the crash reporter.
	 */
	static CORE_API void EnsureFailed( const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* Msg );

private:
	static CORE_API bool VARARGS OptionallyLogFormattedEnsureMessageReturningFalseImpl(bool bLog, const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* FormattedMsg, ...);
	static CORE_API bool OptionallyLogFormattedEnsureMessageReturningFalseImpl(bool bLog, const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* FormattedMsg, va_list Args);

public:
	/**
	 * Logs an error if bLog is true, and returns false.  Takes a formatted string.
	 *
	 * @param	bLog	Log if true.
	 * @param	Expr	Code expression ANSI string (#code)
	 * @param	File	File name ANSI string (__FILE__)
	 * @param	Line	Line number (__LINE__)
	 * @param	FormattedMsg	Informative error message text with variable args
	 *
	 * @return false in all cases.
	 *
	 * Note: this crazy name is to ensure that the crash reporter recognizes it, which checks for functions in the callstack starting with 'EnsureNotFalse'.
	 */

	/** Failed assertion handler.  Warning: May be called at library startup time. */
	template <typename FmtType, typename... Types>
	static FORCEINLINE bool OptionallyLogFormattedEnsureMessageReturningFalse(bool bLog, const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, void* ProgramCounter, const FmtType& FormattedMsg, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to ensureMsgf");

		return OptionallyLogFormattedEnsureMessageReturningFalseImpl(bLog, Expr, File, Line, ProgramCounter, (const TCHAR*)FormattedMsg, Args...);
	}

	static FORCEINLINE bool OptionallyLogFormattedEnsureMessageReturningFalse(bool bLog, const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, void* ProgramCounter, const TCHAR* FormattedMsg, va_list Args)
	{
		return OptionallyLogFormattedEnsureMessageReturningFalseImpl(bLog, Expr, File, Line, ProgramCounter, FormattedMsg, Args);
	}

#endif // DO_CHECK || DO_GUARD_SLOW

	/**
	* Logs an a message to the provided log channel. If a callstack is included (detected by lines starting with 0x) if will be logged in the standard Unreal 
	* format of [Callstack] Address FunctionInfo [File]
	*
	* @param	LogName		Log channel. If NAME_None then LowLevelOutputDebugStringf is used
	* @param	File		File name ANSI string (__FILE__)
	* @param	Line		Line number (__LINE__)
	* @param	Heading		Informative heading displayed above the message callstack
	* @param	Message		Multi-line message with a callstack
	*
	*/
	static CORE_API void LogFormattedMessageWithCallstack(const FName& LogName, const ANSICHAR* File, int32 Line, const TCHAR* Heading, const TCHAR* Message, ELogVerbosity::Type Verbosity);
};

/*----------------------------------------------------------------------------
	Check, verify, etc macros
----------------------------------------------------------------------------*/

//
// "check" expressions are only evaluated if enabled.
// "verify" expressions are always evaluated, but only cause an error if enabled.
//

//#if DO_CHECK || DO_GUARD_SLOW || DO_ENSURE
//	template <typename FmtType, typename... Types>
//	void FORCENOINLINE UE_DEBUG_SECTION FDebug::CheckVerifyFailed(
//		const ANSICHAR* Expr,
//		const ANSICHAR* File,
//		int32 Line,
//		void* ProgramCounter,
//		const FmtType& Format,
//		Types... Args)
//	{
//		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
//		static_assert(TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to CheckVerifyFailed()");
//		return CheckVerifyFailedImpl(Expr, File, Line, ProgramCounter, (const TCHAR*)Format, Args...);
//	}
//#endif

// MSVC (v19.00.24215.1 at time of writing) ignores no-inline attributes on
// lambdas. This can be worked around by calling the lambda from inside this
// templated (and correctly non-inlined) function.
template <typename RetType=void, class InnerType, typename... ArgTypes>
RetType UE_COLD UE_DEBUG_SECTION DispatchCheckVerify (InnerType&& Inner, ArgTypes const&... Args)
{
	return Inner(Args...);
}

#if !UE_BUILD_SHIPPING
#define UE_DEBUG_BREAK_AND_PROMPT_FOR_REMOTE() \
	if (!FPlatformMisc::IsDebuggerPresent()) { FPlatformMisc::PromptForRemoteDebugging(false); } UE_DEBUG_BREAK();
#else
	#define UE_DEBUG_BREAK_AND_PROMPT_FOR_REMOTE()
#endif // !UE_BUILD_SHIPPING

#if !UE_BUILD_SHIPPING
	extern CORE_API bool GIgnoreDebugger;
	// This is named with PLATFORM_ prefix because UE_DEBUG_BREAK* are conditional on a debugger being detected, and PLATFORM_BREAK isn't
	#define PLATFORM_BREAK_IF_DESIRED() if (LIKELY(!GIgnoreDebugger)) { PLATFORM_BREAK(); }
#else
	#define PLATFORM_BREAK_IF_DESIRED() PLATFORM_BREAK();
#endif // !UE_BUILD_SHIPPING


#if DO_CHECK
#ifndef checkCode
	#define checkCode( Code )		do { Code; } while ( false );
#endif
#ifndef verify
	#define verify(expr)			UE_CHECK_IMPL(expr)
#endif
#ifndef check
	#define check(expr)				UE_CHECK_IMPL(expr)
#endif

	// Technically we could use just the _F version (lambda-based) for asserts
	// both with and without formatted messages. However MSVC emits extra
	// unnecessary instructions when using a lambda; hence the Exec() impl.
	#define UE_CHECK_IMPL(expr) \
		{ \
			if(UNLIKELY(!(expr))) \
			{ \
				if (FDebug::CheckVerifyFailedImpl2(#expr, __FILE__, __LINE__, TEXT(""))) \
				{ \
					PLATFORM_BREAK(); \
				} \
				CA_ASSUME(false); \
			} \
			else \
			{ \
				CA_ASSUME(true); \
			} \
		}
	
	/**
	 * verifyf, checkf: Same as verify, check but with printf style additional parameters
	 * Read about __VA_ARGS__ (variadic macros) on http://gcc.gnu.org/onlinedocs/gcc-3.4.4/cpp.pdf.
	 */
#ifndef verifyf
	#define verifyf(expr, format,  ...)		UE_CHECK_F_IMPL(expr, format, ##__VA_ARGS__)
#endif
#ifndef checkf
	#define checkf(expr, format,  ...)		UE_CHECK_F_IMPL(expr, format, ##__VA_ARGS__)
#endif

	#define UE_CHECK_F_IMPL(expr, format, ...) \
		{ \
			if(UNLIKELY(!(expr))) \
			{ \
				UE_VALIDATE_FORMAT_STRING(format, ##__VA_ARGS__); \
				if (FDebug::CheckVerifyFailedImpl2(#expr, __FILE__, __LINE__, format, ##__VA_ARGS__)) \
				{ \
					PLATFORM_BREAK(); \
				} \
				CA_ASSUME(false); \
			} \
			else \
			{ \
				CA_ASSUME(true); \
			} \
		}

	/**
	 * Denotes code paths that should never be reached.
	 */
#ifndef checkNoEntry
	#define checkNoEntry()       check(!"Enclosing block should never be called")
#endif

	/**
	 * Denotes code paths that should not be executed more than once.
	 */
#ifndef checkNoReentry
	#define checkNoReentry()     { static bool s_beenHere##__LINE__ = false;                                         \
	                               check( !"Enclosing block was called more than once" || !s_beenHere##__LINE__ );   \
								   s_beenHere##__LINE__ = true; }
#endif

	class FRecursionScopeMarker
	{
	public: 
		FRecursionScopeMarker(uint16 &InCounter) : Counter( InCounter ) { ++Counter; }
		~FRecursionScopeMarker() { --Counter; }
	private:
		uint16& Counter;
	};

	/**
	 * Denotes code paths that should never be called recursively.
	 */
#ifndef checkNoRecursion
	#define checkNoRecursion()  static uint16 RecursionCounter##__LINE__ = 0;                                            \
	                            check( !"Enclosing block was entered recursively" || RecursionCounter##__LINE__ == 0 );  \
	                            const FRecursionScopeMarker ScopeMarker##__LINE__( RecursionCounter##__LINE__ )
#endif

#ifndef unimplemented
	#define unimplemented()		check(!"Unimplemented function called")
#endif

#else // DO_CHECK
	#define checkCode(...)
	#define check(expr)					{ CA_ASSUME(expr); }
	#define checkf(expr, format, ...)	{ CA_ASSUME(expr); }
	#define checkNoEntry()
	#define checkNoReentry()
	#define checkNoRecursion()
	#define verify(expr)				{ if(UNLIKELY(!(expr))){ CA_ASSUME(false); } }
	#define verifyf(expr, format, ...)	{ if(UNLIKELY(!(expr))){ CA_ASSUME(false); } }
	#define unimplemented()				{ CA_ASSUME(false); }
#endif // DO_CHECK

//
// Check for development only.
//
#if DO_GUARD_SLOW
	#define checkSlow(expr)					check(expr)
	#define checkfSlow(expr, format, ...)	checkf(expr, format, ##__VA_ARGS__)
	#define verifySlow(expr)				check(expr)
#else
	#define checkSlow(expr)					{ CA_ASSUME(expr); }
	#define checkfSlow(expr, format, ...)	{ CA_ASSUME(expr); }
	#define verifySlow(expr)				{ if(UNLIKELY(!(expr))) { CA_ASSUME(false); } }
#endif

/**
 * ensure() can be used to test for *non-fatal* errors at runtime
 *
 * Rather than crashing, an error report (with a full call stack) will be logged and submitted to the crash server. 
 * This is useful when you want runtime code verification but you're handling the error case anyway.
 *
 * Note: ensure() can be nested within conditionals!
 *
 * Example:
 *
 *		if (ensure(InObject != nullptr))
 *		{
 *			InObject->Modify();
 *		}
 *
 * This code is safe to execute as the pointer dereference is wrapped in a non-nullptr conditional block, but
 * you still want to find out if this ever happens so you can avoid side effects.  Using ensure() here will
 * force a crash report to be generated without crashing the application (and potentially causing editor
 * users to lose unsaved work.)
 *
 * ensure() resolves to just evaluate the expression when DO_CHECK is 0 (typically shipping or test builds).
 *
 * By default a given call site will only print the callstack and submit the 'crash report' the first time an
 * ensure is hit in a session; ensureAlways can be used instead if you want to handle every failure
 */

#if DO_ENSURE && !USING_CODE_ANALYSIS // The Visual Studio 2013 analyzer doesn't understand these complex conditionals
	namespace UE::Assert::Private
	{
	
	/** Data about an ensure that is constant for every occurrence. */
	struct FStaticEnsureRecord
	{
		const TCHAR* Format = nullptr;
		const ANSICHAR* Expression = nullptr;
		const ANSICHAR* File = nullptr;
		int32 Line = 0;
		bool bAlways = false;

		// Workaround for https://developercommunity.visualstudio.com/t/Incorrect-warning-C4700-with-unrelated-s/10285950
		constexpr FStaticEnsureRecord(
			const TCHAR* InFormat,
			const ANSICHAR* InExpression,
			const ANSICHAR* InFile,
			int32 InLine,
			bool bInAlways)
			: Format(InFormat)
			, Expression(InExpression)
			, File(InFile)
			, Line(InLine)
			, bAlways(bInAlways)
		{
		}
	};

	CORE_API bool UE_COLD UE_DEBUG_SECTION CheckEnsureFailed(bool bAlways, std::atomic<uint8>& bExecuted, const ANSICHAR* File, int32 Line, const ANSICHAR* Expr);
	CORE_API bool UE_COLD UE_DEBUG_SECTION CheckEnsureFailed(bool bAlways, const std::atomic<uint8>& bExecuted);
	CORE_API bool UE_COLD UE_DEBUG_SECTION VARARGS EnsureFailed(std::atomic<uint8>& bExecuted, const FStaticEnsureRecord* Ensure, ...);

	} // UE::Assert::Private

	template <uint64 Uid>
	std::atomic<uint8> bGEnsureHasExecuted = false;

	UE_NODEBUG constexpr uint64 FileLineHashForEnsure(const char* Filename, uint32 Line)
	{
		// Based on djb2
		uint32 Result = 5381;
		while (char Ch = *Filename++)
		{
			Result = ((Result << 5) + Result) + Ch;
		}
		return ((uint64)Result << 32) | (uint64)Line;
	}

// UE_BREAK_AND_RETURN_FALSE() - an expression which breaks into the debugger and returns false.
#if PLATFORM_BREAK_IS_EXPRESSION
	// If PLATFORM_BREAK is also an expression, it can be used directly.
	#define UE_BREAK_AND_RETURN_FALSE() (PLATFORM_BREAK(), false)
#elif defined(__clang__) || defined(__GNUC__)
	// Clang and GCC support 'statement expressions' which allows the expansion of statements in an
	// expression context.
	#define UE_BREAK_AND_RETURN_FALSE() ({ PLATFORM_BREAK(); false; })
#else
	// Fallback to using a lambda if there is no other choice, which generates more debug information.
	#define UE_BREAK_AND_RETURN_FALSE() [](){ PLATFORM_BREAK(); return false; }()
#endif

	#define UE_ENSURE_IMPL(Always, InExpression) \
		( \
			LIKELY(!!(InExpression)) || \
			( \
				::UE::Assert::Private::CheckEnsureFailed( \
					Always, \
					::bGEnsureHasExecuted<FileLineHashForEnsure(__FILE__, __LINE__)>, \
					__FILE__, \
					__LINE__, \
					#InExpression \
				) && \
				UE_BREAK_AND_RETURN_FALSE() \
			) \
		)

	#define UE_ENSURE_IMPL2(Always, InExpression, InFormat, ...) \
		( \
			LIKELY(!!(InExpression)) || \
			[&]() UE_COLD UE_DEBUG_SECTION \
			{ \
				UE_VALIDATE_FORMAT_STRING(InFormat, ##__VA_ARGS__); \
				std::atomic<uint8>& bEnsureHasExecuted = ::bGEnsureHasExecuted < FileLineHashForEnsure(__FILE__, __LINE__) >; \
				static constexpr ::UE::Assert::Private::FStaticEnsureRecord ENSURE_Static(InFormat, #InExpression, __builtin_FILE(), __builtin_LINE(), Always); \
				if (::UE::Assert::Private::CheckEnsureFailed(Always, bEnsureHasExecuted) && ::UE::Assert::Private::EnsureFailed(bEnsureHasExecuted, &ENSURE_Static, ##__VA_ARGS__)) \
				{ \
					PLATFORM_BREAK(); \
				} \
				return false; \
			}() \
		)

	#define ensure(           InExpression                ) UE_ENSURE_IMPL (false, InExpression)
	#define ensureMsgf(       InExpression, InFormat, ... ) UE_ENSURE_IMPL2(false, InExpression, InFormat, ##__VA_ARGS__)
	#define ensureAlways(     InExpression                ) UE_ENSURE_IMPL (true,  InExpression)
	#define ensureAlwaysMsgf( InExpression, InFormat, ... ) UE_ENSURE_IMPL2(true,  InExpression, InFormat, ##__VA_ARGS__)


#else // DO_ENSURE && !USING_CODE_ANALYSIS

	#define ensure(           InExpression                ) (LIKELY(!!(InExpression)))
	#define ensureMsgf(       InExpression, InFormat, ... ) (LIKELY(!!(InExpression)))
	#define ensureAlways(     InExpression                ) (LIKELY(!!(InExpression)))
	#define ensureAlwaysMsgf( InExpression, InFormat, ... ) (LIKELY(!!(InExpression)))

#endif	// DO_ENSURE && !USING_CODE_ANALYSIS

namespace UEAsserts_Private
{
	// A junk function to allow us to use sizeof on a member variable which is potentially a bitfield
	template <typename T>
	bool GetMemberNameCheckedJunk(const T&);
	template <typename T>
	bool GetMemberNameCheckedJunk(const volatile T&);
	template <typename R, typename ...Args>
	bool GetMemberNameCheckedJunk(R(*)(Args...));
}

// Returns FName(ANSITEXTVIEW("EnumeratorName")), while statically verifying that the enumerator exists in the enum
#define GET_ENUMERATOR_NAME_CHECKED(EnumName, EnumeratorName) \
	(FName(GET_ENUMERATOR_NAME_ANSI_STRING_VIEW_CHECKED(EnumName, EnumeratorName)))

#define GET_ENUMERATOR_NAME_STRING_CHECKED(EnumName, EnumeratorName) \
	((void)sizeof(UEAsserts_Private::GetMemberNameCheckedJunk(EnumName::EnumeratorName)), TEXT(#EnumeratorName))

#define GET_ENUMERATOR_NAME_STRING_VIEW_CHECKED(EnumName, EnumeratorName) \
	((void)sizeof(UEAsserts_Private::GetMemberNameCheckedJunk(EnumName::EnumeratorName)), TEXTVIEW(#EnumeratorName))

#define GET_ENUMERATOR_NAME_ANSI_STRING_CHECKED(EnumName, EnumeratorName) \
	((void)sizeof(UEAsserts_Private::GetMemberNameCheckedJunk(EnumName::EnumeratorName)), #EnumeratorName)

#define GET_ENUMERATOR_NAME_ANSI_STRING_VIEW_CHECKED(EnumName, EnumeratorName) \
	((void)sizeof(UEAsserts_Private::GetMemberNameCheckedJunk(EnumName::EnumeratorName)), ANSITEXTVIEW(#EnumeratorName))

// Returns FName(ANSITEXTVIEW("MemberName")), while statically verifying that the member exists in ClassName
#define GET_MEMBER_NAME_CHECKED(ClassName, MemberName) \
	(FName(GET_MEMBER_NAME_ANSI_STRING_VIEW_CHECKED(ClassName, MemberName)))

#define GET_MEMBER_NAME_STRING_CHECKED(ClassName, MemberName) \
	((void)sizeof(UEAsserts_Private::GetMemberNameCheckedJunk(((ClassName*)0)->MemberName)), TEXT(#MemberName))

#define GET_MEMBER_NAME_STRING_VIEW_CHECKED(ClassName, MemberName) \
	((void)sizeof(UEAsserts_Private::GetMemberNameCheckedJunk(((ClassName*)0)->MemberName)), TEXTVIEW(#MemberName))

#define GET_MEMBER_NAME_ANSI_STRING_CHECKED(ClassName, MemberName) \
	((void)sizeof(UEAsserts_Private::GetMemberNameCheckedJunk(((ClassName*)0)->MemberName)), #MemberName)

#define GET_MEMBER_NAME_ANSI_STRING_VIEW_CHECKED(ClassName, MemberName) \
	((void)sizeof(UEAsserts_Private::GetMemberNameCheckedJunk(((ClassName*)0)->MemberName)), ANSITEXTVIEW(#MemberName))

// Returns FName(ANSITEXTVIEW("FunctionName")), while statically verifying that the function exists in ClassName
#define GET_FUNCTION_NAME_CHECKED(ClassName, FunctionName) \
	(FName(GET_FUNCTION_NAME_ANSI_STRING_VIEW_CHECKED(ClassName, FunctionName)))

#define GET_FUNCTION_NAME_STRING_CHECKED(ClassName, FunctionName) \
	((void)sizeof(&ClassName::FunctionName), TEXT(#FunctionName))

#define GET_FUNCTION_NAME_STRING_VIEW_CHECKED(ClassName, FunctionName) \
	((void)sizeof(&ClassName::FunctionName), TEXTVIEW(#FunctionName))

#define GET_FUNCTION_NAME_ANSI_STRING_CHECKED(ClassName, FunctionName) \
	((void)sizeof(&ClassName::FunctionName), #FunctionName)

#define GET_FUNCTION_NAME_ANSI_STRING_VIEW_CHECKED(ClassName, FunctionName) \
	((void)sizeof(&ClassName::FunctionName), ANSITEXTVIEW(#FunctionName))

// Returns FName("FunctionName"), while statically verifying that the function exists in ClassName
// Handles overloaded functions by specifying the argument of the overload to use
#define GET_FUNCTION_NAME_CHECKED_OneParam(ClassName, FunctionName, ArgType) \
	(FName(GET_FUNCTION_NAME_ANSI_STRING_CHECKED_OneParam(ClassName, FunctionName, ArgType)))

#define GET_FUNCTION_NAME_CHECKED_TwoParams(ClassName, FunctionName, ArgType1, ArgType2) \
	(FName(GET_FUNCTION_NAME_ANSI_STRING_CHECKED_TwoParams(ClassName, FunctionName, ArgType1, ArgType2)))

#define GET_FUNCTION_NAME_CHECKED_ThreeParams(ClassName, FunctionName, ArgType1, ArgType2, ArgType3) \
	(FName(GET_FUNCTION_NAME_ANSI_STRING_CHECKED_ThreeParams(ClassName, FunctionName, ArgType1, ArgType2, ArgType3)))

#define GET_FUNCTION_NAME_CHECKED_FourParams(ClassName, FunctionName, ArgType1, ArgType2, ArgType3, ArgType4) \
	(FName(GET_FUNCTION_NAME_ANSI_STRING_CHECKED_FourParams(ClassName, FunctionName, ArgType1, ArgType2, ArgType3, ArgType4)))

#define GET_FUNCTION_NAME_STRING_CHECKED_OneParam(ClassName, FunctionName, ArgType) \
	((void)sizeof((std::declval<ClassName&>().FunctionName(std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType)>()), (int)0)), TEXT(#FunctionName))

#define GET_FUNCTION_NAME_STRING_CHECKED_TwoParams(ClassName, FunctionName, ArgType1, ArgType2) \
	((void)sizeof((std::declval<ClassName&>().FunctionName(std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType1)>(), std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType2)>()), (int)0)), TEXT(#FunctionName))

#define GET_FUNCTION_NAME_STRING_CHECKED_ThreeParams(ClassName, FunctionName, ArgType1, ArgType2, ArgType3) \
	((void)sizeof((std::declval<ClassName&>().FunctionName(std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType1)>(), std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType2)>(), std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType3)>()), (int)0)), TEXT(#FunctionName))

#define GET_FUNCTION_NAME_STRING_CHECKED_FourParams(ClassName, FunctionName, ArgType1, ArgType2, ArgType3, ArgType4) \
	((void)sizeof((std::declval<ClassName&>().FunctionName(std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType1)>(), std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType2)>(), std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType3)>(), std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType4)>()), (int)0)), TEXT(#FunctionName))

#define GET_FUNCTION_NAME_ANSI_STRING_CHECKED_OneParam(ClassName, FunctionName, ArgType) \
	((void)sizeof((std::declval<ClassName&>().FunctionName(std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType)>()), (int)0)), #FunctionName)

#define GET_FUNCTION_NAME_ANSI_STRING_CHECKED_TwoParams(ClassName, FunctionName, ArgType1, ArgType2) \
	((void)sizeof((std::declval<ClassName&>().FunctionName(std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType1)>(), std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType2)>()), (int)0)), #FunctionName)

#define GET_FUNCTION_NAME_ANSI_STRING_CHECKED_ThreeParams(ClassName, FunctionName, ArgType1, ArgType2, ArgType3) \
	((void)sizeof((std::declval<ClassName&>().FunctionName(std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType1)>(), std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType2)>(), std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType3)>()), (int)0)), #FunctionName)

#define GET_FUNCTION_NAME_ANSI_STRING_CHECKED_FourParams(ClassName, FunctionName, ArgType1, ArgType2, ArgType3, ArgType4) \
	((void)sizeof((std::declval<ClassName&>().FunctionName(std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType1)>(), std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType2)>(), std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType3)>(), std::declval<UE_REMOVE_OPTIONAL_PARENS(ArgType4)>()), (int)0)), #FunctionName)

/*----------------------------------------------------------------------------
	Low level error macros
----------------------------------------------------------------------------*/

namespace UE::Assert::Private
{
	CORE_API void UE_COLD UE_DEBUG_SECTION VARARGS ProcessLowLevelFatalError(const ANSICHAR* File, int32 Line, const TCHAR* Format, ...);
	CORE_API void UE_COLD UE_DEBUG_SECTION VARARGS ProcessLowLevelFatalError(const ANSICHAR* File, int32 Line, const ANSICHAR* Format, ...);
	CORE_API void UE_COLD UE_DEBUG_SECTION VARARGS ProcessLowLevelFatalError(const ANSICHAR* File, int32 Line, const UTF8CHAR* Format, ...);
} // UE::Assert::Private

template <typename... ArgTypes>
UE_DEPRECATED(5.7, "Use LowLevelFatalError.")
UE_REWRITE void LowLevelFatalErrorHandler(const ANSICHAR* File, int32 Line, const TCHAR* Format = TEXT(""), ArgTypes... Args)
{
	UE::Assert::Private::ProcessLowLevelFatalError(File, Line, Format, Args...);
}

#define LowLevelFatalError(Format, ...) \
	{ \
		UE_VALIDATE_FORMAT_STRING(Format, ##__VA_ARGS__); \
		::UE::Assert::Private::ProcessLowLevelFatalError(__FILE__, __LINE__, Format, ##__VA_ARGS__); \
	}
