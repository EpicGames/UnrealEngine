// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"

class IAsyncTaskNotificationImpl;
struct FLogCategoryBase;
struct FSlateBrush;

/**
 * Configuration data for initializing an asynchronous task notification.
 */
struct FAsyncTaskNotificationConfig
{
	/** The title text displayed in the notification */
	FText TitleText;

	/** The progress text displayed in the notification (if any) */
	FText ProgressText;

	/** The fade in duration of the notification */
	float FadeInDuration = 0.5f;

	/** The fade out duration of the notification */
	float FadeOutDuration = 2.0f;

	/** The duration before a fadeout for the notification */
	float ExpireDuration = 1.0f;

	/** Should this notification be "headless"? (ie, not display any UI) */
	bool bIsHeadless = false;

	/** Can this task be canceled? Will show a cancel button for in-progress tasks (attribute queried from the game thread) */
	TAttribute<bool> bCanCancel = false;

	/** Keep this notification open on success? Will show a close button (attribute queried from the game thread) */
	TAttribute<bool> bKeepOpenOnSuccess = false;

	/** Keep this notification open on failure? Will show an close button (attribute queried from the game thread) */
	TAttribute<bool> bKeepOpenOnFailure = false;

	/** The icon image to display next to the text, or null to use the default icon */
	const FSlateBrush* Icon = nullptr;

	/** Category this task should log its notifications under, or null to skip logging */
	const FLogCategoryBase* LogCategory = nullptr;
};

enum class EAsyncTaskNotificationState : uint8
{
	None = 0,
	Pending,
	Success,
	Failure,
	Prompt,
};

enum class EAsyncTaskNotificationPromptAction : uint8
{
	/** No action taken for the ongoing task. */
	None = 0,
	/** Continue ongoing task after a prompt. */
	Continue,
	/** Cancel ongoing task. */
	Cancel,
	/** No action can be taken, task is unattended. */
	Unattended
};

/**
 * Async Notification State data
 */
struct FAsyncNotificationStateData
{
	FAsyncNotificationStateData()
		: State(EAsyncTaskNotificationState::Pending)
	{}

	FAsyncNotificationStateData(const FText& InTitleText, const FText& InProgressText, EAsyncTaskNotificationState InState)
		: State(InState)
		, TitleText(InTitleText)
		, ProgressText(InProgressText)
	{}

	/** The notification state. */
	EAsyncTaskNotificationState State;

	/** The title text displayed in the notification */
	FText TitleText;

	/** The progress text displayed in the notification (if any) */
	FText ProgressText;

	/** The prompt text displayed in the notification, follow the progress text in console notification, used on button for UI notification (if any) */
	FText PromptText;

	/** Text to display for the hyperlink message. */
	FText HyperlinkText;

	/** Hyperlink callback. If not bound the hyperlink text won't be displayed. */
	FSimpleDelegate Hyperlink;
};

/**
 * Provides notifications for an on-going asynchronous task.
 */
class FAsyncTaskNotification
{
public:
	/**
	 * Create an asynchronous task notification.
	 */
	CORE_API FAsyncTaskNotification(const FAsyncTaskNotificationConfig& InConfig);

	/**
	 * Destroy the asynchronous task notification.
	 */
	CORE_API ~FAsyncTaskNotification();

	/**
	 * Non-copyable.
	 */
	FAsyncTaskNotification(const FAsyncTaskNotification&) = delete;
	FAsyncTaskNotification& operator=(const FAsyncTaskNotification&) = delete;

	/**
	 * Movable.
	 */
	CORE_API FAsyncTaskNotification(FAsyncTaskNotification&& InOther);
	CORE_API FAsyncTaskNotification& operator=(FAsyncTaskNotification&& InOther);

	/**
	 * Set the title text of this notification.
	 */
	CORE_API void SetTitleText(const FText& InTitleText, const bool bClearProgressText = true);

	/**
	 * Set the progress text of this notification.
	 */
	CORE_API void SetProgressText(const FText& InProgressText);

	/**
	 * Set the prompt text of this notification, if needed.
	 */
	CORE_API void SetPromptText(const FText& InPromptText);

	/**
	 * Set the Hyperlink for this notification.
	 */
	CORE_API void SetHyperlink(const FSimpleDelegate& InHyperlink, const FText& InHyperlinkText = FText());

	/**
	 * Set the task as complete.
	 */
	CORE_API void SetComplete(const bool bSuccess = true);

	/**
	 * Update the text and set the task as complete.
	 */
	CORE_API void SetComplete(const FText& InTitleText, const FText& InProgressText, const bool bSuccess = true);

	/**
	 * Set the notification state. Provide finer control than SetComplete by setting every field of the state.
	 */
	CORE_API void SetNotificationState(const FAsyncNotificationStateData& InState);

	/**
	 * Set whether this task be canceled.
	 */
	CORE_API void SetCanCancel(const TAttribute<bool>& InCanCancel);

	/**
	 * Set whether to keep this notification open on success.
	*/
	CORE_API void SetKeepOpenOnSuccess(const TAttribute<bool>& InKeepOpenOnSuccess);

	/**
	 * Set whether to keep this notification open on failure.
	 */
	CORE_API void SetKeepOpenOnFailure(const TAttribute<bool>& InKeepOpenOnFailure);

	/**
	 * Return the notification prompt action.
	 * The action resets to `None` when the notification state changes.
	 */
	CORE_API EAsyncTaskNotificationPromptAction GetPromptAction() const;

private:
	/** Pointer to the real notification implementation */
	TSharedPtr<IAsyncTaskNotificationImpl> NotificationImpl;
};
