// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/TVariant.h"
#include "Templates/Function.h"
#include "Templates/Identity.h"
#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"

/**
 * Attribute object
 */
template< typename ObjectType >
class TAttribute
{

public:

	/**
	 * Attribute 'getter' delegate
	 *
	 * ObjectType GetValue() const
	 *
	 * @return  The attribute's value
	 */
	DECLARE_DELEGATE_RetVal( ObjectType, FGetter );


	/** Default constructor. */
	TAttribute()
		: Value()         // NOTE: Potentially uninitialized for atomics!!
		, bIsSet(false)
		, Getter()
	{
	}

	/**
	 * Construct implicitly from an initial value
	 *
	 * @param  InInitialValue  The value for this attribute
	 */
	template< typename OtherType >
	TAttribute( const OtherType& InInitialValue )
		: Value( static_cast<ObjectType>(InInitialValue) )
		, bIsSet(true)
		, Getter()
	{
	}

	/** 
	 * Construct implicitly from moving an initial value
	 *
	 * @param InInitialValue
	 */
	TAttribute( ObjectType&& InInitialValue)
		: Value(MoveTemp(InInitialValue))
		, bIsSet(true)
		, Getter()
	{
	}

	/**
	 * Constructs by binding an arbitrary function that will be called to generate this attribute's value on demand. 
	 * After binding, the attribute will no longer have a value that can be accessed directly, and instead the bound
	 * function will always be called to generate the value.
	 *
	 * @param  InUserObject  Shared Pointer to the instance of the class that contains the member function you want to bind.  The attribute will only retain a weak pointer to this class.
	 * @param  InMethodPtr Member function to bind.  The function's structure (return value, arguments, etc) must match IBoundAttributeDelegate's definition.
	 */
	template< class SourceType >
	TAttribute( TSharedRef< SourceType > InUserObject, typename FGetter::template TConstMethodPtr< SourceType > InMethodPtr )
		: Value()
		, bIsSet(true)
		, Getter(FGetter::CreateSP(InUserObject, InMethodPtr))
	{
	}

	/**
	 * Constructs by binding an arbitrary function that will be called to generate this attribute's value on demand. 
	 * After binding, the attribute will no longer have a value that can be accessed directly, and instead the bound
	 * function will always be called to generate the value.
	 *
	 * @param  InUserObject  Shared Pointer to the instance of the class that contains the member function you want to bind.  The attribute will only retain a weak pointer to this class.
	 * @param  InMethodPtr Member function to bind.  The function's structure (return value, arguments, etc) must match IBoundAttributeDelegate's definition.
	 */
	template< class SourceType >
	TAttribute( SourceType* InUserObject, typename FGetter::template TConstMethodPtr< SourceType > InMethodPtr )
		: Value()
		, bIsSet(true)
		, Getter(FGetter::CreateSP(InUserObject, InMethodPtr))
	{
	}

	/**
	 * Static: Creates an attribute that's pre-bound to the specified 'getter' delegate
	 *
	 * @param  InGetter		Delegate to bind
	 */
	[[nodiscard]] static TAttribute Create( const FGetter& InGetter )
	{
		const bool bExplicitConstructor = true;
		return TAttribute( InGetter, bExplicitConstructor );
	}

	/**
	 * Static: Creates an attribute that's pre-bound to the specified 'getter' delegate
	 *
	 * @param  InGetter		Delegate to bind
	 */
	[[nodiscard]] static TAttribute Create(FGetter&& InGetter)
	{
		const bool bExplicitConstructor = true;
		return TAttribute(MoveTemp(InGetter), bExplicitConstructor);
	}

	/**
	 * Creates an attribute by binding an arbitrary function that will be called to generate this attribute's value on demand.
	 * After binding, the attribute will no longer have a value that can be accessed directly, and instead the bound
	 * function will always be called to generate the value.
	 *
	 * @param  InFuncPtr Member function to bind.  The function's structure (return value, arguments, etc) must match IBoundAttributeDelegate's definition.
	 */
	template <typename FuncPtrType, typename... VarTypes >
	[[nodiscard]] static TAttribute CreateStatic(FuncPtrType&& InFuncPtr, VarTypes... Vars)
	{
		const bool bExplicitConstructor = true;
		return TAttribute(FGetter::CreateStatic(InFuncPtr, MoveTemp(Vars)...), bExplicitConstructor);
	}

	/**
	 * Helper function for creating TAttributes from a const member function pointer, accessed through a raw pointer
	 */
	template<typename SourceType, typename SourceTypeOrBase, typename... PayloadTypes>
	[[nodiscard]] UE_FORCEINLINE_HINT static TAttribute CreateRaw(const SourceType* InObject, ObjectType (SourceTypeOrBase::*InMethod)(PayloadTypes...) const, typename TDecay<PayloadTypes>::Type... InputPayload)
	{
		return Create(FGetter::CreateRaw(InObject, InMethod, MoveTemp(InputPayload)...));
	}

	/**
	 * Helper function for creating TAttributes from a const member function pointer, accessed through a weak pointer to the shared object
	 */
	template<typename SourceType, typename SourceTypeOrBase, typename... PayloadTypes>
	[[nodiscard]] UE_FORCEINLINE_HINT static TAttribute CreateSP(const SourceType* InObject, ObjectType (SourceTypeOrBase::*InMethod)(PayloadTypes...) const, typename TDecay<PayloadTypes>::Type... InputPayload)
	{
		return Create(FGetter::CreateSP(InObject, InMethod, MoveTemp(InputPayload)...));
	}

	/**
	 * Helper function for creating TAttributes from a lambda
	 * TAttribute<float> FloatAttribute = TAttribute<float>::CreateLambda([]{ return 10.f; });
	 */
	template<typename LambdaType, typename... PayloadTypes>
	[[nodiscard]] UE_FORCEINLINE_HINT static TAttribute CreateLambda(LambdaType&& InCallable, PayloadTypes&&... InputPayload)
	{
		return Create(FGetter::CreateLambda(Forward<LambdaType>(InCallable), Forward<PayloadTypes>(InputPayload)...));
	}

	/**
	 * Helper function for creating TAttributes from a weak shared pointer C++ lambda delegate
	 */
	template<typename UserClass, ESPMode Mode, typename LambdaType, typename... PayloadTypes>
	[[nodiscard]] UE_FORCEINLINE_HINT static TAttribute CreateSPLambda(const TSharedRef<UserClass, Mode>& InUserObjectRef, LambdaType&& InCallable, PayloadTypes&&... InputPayload)
	{
		return Create(FGetter::CreateSPLambda(InUserObjectRef, Forward<LambdaType>(InCallable), Forward<PayloadTypes>(InputPayload)...));
	}
	template <typename UserClass, typename LambdaType, typename... PayloadTypes>
	[[nodiscard]] UE_FORCEINLINE_HINT static TAttribute CreateSPLambda(UserClass* InUserObject, LambdaType&& InCallable, PayloadTypes&&... InputPayload)
	{
		return Create(FGetter::CreateSPLambda(InUserObject, Forward<LambdaType>(InCallable), Forward<PayloadTypes>(InputPayload)...));
	}

	/**
	 * Helper function for creating TAttributes from a weak object pointer C++ lambda delegate
	 */
	template <typename UserClass, typename LambdaType, typename... PayloadTypes>
	[[nodiscard]] UE_FORCEINLINE_HINT static TAttribute CreateWeakLambda(UserClass* InUserObject, LambdaType&& InCallable, PayloadTypes&&... InputPayload)
	{
		return Create(FGetter::CreateWeakLambda(InUserObject, Forward<LambdaType>(InCallable), Forward<PayloadTypes>(InputPayload)...));
	}

	/**
	 * Creates an attribute by binding an arbitrary function that will be called to generate this attribute's value on demand.
	 * After binding, the attribute will no longer have a value that can be accessed directly, and instead the bound
	 * function will always be called to generate the value.
	 *
	 * @param  InUserObject  Instance of the class that contains the member function you want to bind.
	 * @param  InFunctionName Member function name to bind.
	 */
	template< class SourceType >
	[[nodiscard]] static TAttribute< ObjectType > Create(SourceType* InUserObject, const FName& InFunctionName)
	{
		TAttribute< ObjectType > Attrib;
		Attrib.BindUFunction<SourceType>(InUserObject, InFunctionName);
		return Attrib;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT static TAttribute< ObjectType > Create(TFunction<ObjectType(void)>&& InLambda)
	{
		return Create(TAttribute< ObjectType >::FGetter::CreateLambda(MoveTemp(InLambda)));
	}

	/**
	 * Sets the attribute's value
	 *
	 * @param  InNewValue  The value to set the attribute to
	 */
	template< typename OtherType >
	void Set( const OtherType& InNewValue )
	{
		Getter.Unbind();
		Value = InNewValue;
		bIsSet = true;
	}

	/**
	 * Sets the attribute's value
	 * 
	 * @param InNewValue  The value to set the attribute to
	 */
	void Set( ObjectType&& InNewValue )
	{
		Getter.Unbind();
		Value = MoveTemp(InNewValue);
		bIsSet = true;
	}

	/** Was this TAttribute ever assigned? */
	bool IsSet() const
	{
		return bIsSet;
	}

	/**
	 * Gets the attribute's current value.
	 * Assumes that the attribute is set.
	 *
	 * @return  The attribute's value
	 */
	const ObjectType& Get() const
	{
		// If we have a getter delegate, then we'll call that to generate the value
		if( Getter.IsBound() )
		{
			// Call the delegate to get the value.  Note that this will assert if the delegate is not currently
			// safe to call (e.g. object was deleted and we could detect that)

			// NOTE: We purposely overwrite our value copy here so that we can return the value by address in
			// the most common case, which is an attribute that doesn't have a delegate bound to it at all.
			Value = Getter.Execute();
		}

		// Return the stored value
		return Value;
	}

	/**
	 * Gets the attribute's current value. The attribute may not be set, in which case use the default value provided.
	 * Shorthand for the boilerplate code: MyAttribute.IsSet() ? MyAttribute.Get() : DefaultValue
	 */
	const ObjectType& Get( const ObjectType& DefaultValue ) const
	{
		return bIsSet ? Get() : DefaultValue;
	}

	/**
	 * Binds an arbitrary function that will be called to generate this attribute's value on demand.  After
	 * binding, the attribute will no longer have a value that can be accessed directly, and instead the bound
	 * function will always be called to generate the value.
	 *
	 * @param  InGetter  The delegate object with your function binding
	 */
	void Bind( const FGetter& InGetter )
	{
		bIsSet = true;
		Getter = InGetter;
	}
	

	/**
	 * Binds an arbitrary function that will be called to generate this attribute's value on demand.  After
	 * binding, the attribute will no longer have a value that can be accessed directly, and instead the bound
	 * function will always be called to generate the value.
	 *
	 * @param  InGetter  The delegate object with your function binding
	 */
	void Bind( FGetter&& InGetter )
	{
		bIsSet = true;
		Getter = MoveTemp(InGetter);
	}

	/**
	 * Binds an arbitrary function that will be called to generate this attribute's value on demand. 
	 * After binding, the attribute will no longer have a value that can be accessed directly, and instead the bound
	 * function will always be called to generate the value.
	 *
	 * @param  InFuncPtr	Function to bind.  The function's structure (return value, arguments, etc) must match IBoundAttributeDelegate's definition.
	 */
	template < typename... VarTypes >
	void BindStatic( TIdentity_T< typename FGetter::template TFuncPtr< std::decay_t<VarTypes>... > > InFuncPtr, VarTypes&&... Vars )
	{
		bIsSet = true;
		Getter.BindStatic( InFuncPtr, Forward<VarTypes>(Vars)... );
	}
	
	/**
	 * Binds an arbitrary function that will be called to generate this attribute's value on demand. 
	 * After binding, the attribute will no longer have a value that can be accessed directly, and instead the bound
	 * function will always be called to generate the value.
	 *
	 * @param  InUserObject  Instance of the class that contains the member function you want to bind.
	 * @param  InMethodPtr Member function to bind.  The function's structure (return value, arguments, etc) must match IBoundAttributeDelegate's definition.
	 */
	template< class SourceType, typename... VarTypes >
	void BindRaw( SourceType* InUserObject, typename FGetter::template TConstMethodPtr< SourceType, std::decay_t<VarTypes>... > InMethodPtr, VarTypes&&... Vars )
	{
		bIsSet = true;
		Getter.BindRaw( InUserObject, InMethodPtr, Forward<VarTypes>(Vars)... );
	}

	/**
	 * Binds an arbitrary function that will be called to generate this attribute's value on demand. 
	 * After binding, the attribute will no longer have a value that can be accessed directly, and instead the bound
	 * function will always be called to generate the value.
	 *
	 * @param  InUserObject  Shared Pointer to the instance of the class that contains the member function you want to bind.  The attribute will only retain a weak pointer to this class.
	 * @param  InMethodPtr Member function to bind.  The function's structure (return value, arguments, etc) must match IBoundAttributeDelegate's definition.
	 */
	template< class SourceType, typename... VarTypes >
	void Bind( TSharedRef< SourceType > InUserObject, typename FGetter::template TConstMethodPtr< SourceType, std::decay_t<VarTypes>... > InMethodPtr, VarTypes&&... Vars )
	{
		bIsSet = true;
		Getter.BindSP( InUserObject, InMethodPtr, Forward<VarTypes>(Vars)... );
	}

	/**
	 * Binds an arbitrary function that will be called to generate this attribute's value on demand. 
	 * After binding, the attribute will no longer have a value that can be accessed directly, and instead the bound
	 * function will always be called to generate the value.
	 *
	 * @param  InUserObject  Shared Pointer to the instance of the class that contains the member function you want to bind.  The attribute will only retain a weak pointer to this class.
	 * @param  InMethodPtr Member function to bind.  The function's structure (return value, arguments, etc) must match IBoundAttributeDelegate's definition.
	 */
	template< class SourceType, typename... VarTypes >
	void Bind( SourceType* InUserObject, typename FGetter::template TConstMethodPtr< SourceType, std::decay_t<VarTypes>... > InMethodPtr, VarTypes&&... Vars )
	{
		bIsSet = true;
		Getter.BindSP( InUserObject, InMethodPtr, Forward<VarTypes>(Vars)... );
	}

	/**
	 * Binds an arbitrary function that will be called to generate this attribute's value on demand. 
	 * After binding, the attribute will no longer have a value that can be accessed directly, and instead the bound
	 * function will always be called to generate the value.
	 *
	 * @param  InUserObject  Instance of the class that contains the member function you want to bind.
	 * @param  InMethodPtr Member function to bind.  The function's structure (return value, arguments, etc) must match IBoundAttributeDelegate's definition.
	 */
	template< class SourceType, typename... VarTypes >	
	void BindUObject( SourceType* InUserObject, typename FGetter::template TConstMethodPtr< SourceType, std::decay_t<VarTypes>... > InMethodPtr, VarTypes&&... Vars)
	{
		bIsSet = true;
		Getter.BindUObject( InUserObject, InMethodPtr, Forward<VarTypes>(Vars)... );
	}

	/**
	 * Binds an arbitrary function that will be called to generate this attribute's value on demand.
	 * After binding, the attribute will no longer have a value that can be accessed directly, and instead the bound
	 * function will always be called to generate the value.
	 *
	 * @param  InUserObject  Instance of the class that contains the member function you want to bind.
	 * @param  InFunctionName Member function name to bind.
	 */
	template< class SourceType >
	void BindUFunction( SourceType* InUserObject, const FName& InFunctionName )
	{
		bIsSet = true;
		Getter.BindUFunction(InUserObject, InFunctionName);
	}

	/**
	 * Static: Binds a C++ lambda delegate
	 * technically this works for any functor types, but lambdas are the primary use case
	 */
	template<typename LambdaType, typename... PayloadTypes>
	inline void BindLambda(LambdaType&& InCallable, PayloadTypes&&... InputPayload)
	{
		bIsSet = true;
		Getter.BindLambda(Forward<LambdaType>(InCallable), Forward<PayloadTypes>(InputPayload)...);
	}

	/**
	 * Static: Binds a weak shared pointer C++ lambda delegate
	 * technically this works for any functor types, but lambdas are the primary use case
	 */
	template<typename UserClass, ESPMode Mode, typename LambdaType, typename... PayloadTypes>
	inline void BindSPLambda(const TSharedRef<UserClass, Mode>& InUserObjectRef, LambdaType&& InCallable, PayloadTypes&&... InputPayload)
	{
		bIsSet = true;
		Getter.BindSPLambda(InUserObjectRef, Forward<LambdaType>(InCallable), Forward<PayloadTypes>(InputPayload)...);
	}
	template <typename UserClass, typename LambdaType, typename... PayloadTypes>
	inline void BindSPLambda(const UserClass* InUserObject, LambdaType&& InCallable, PayloadTypes&&... InputPayload)
	{
		bIsSet = true;
		Getter.BindSPLambda(InUserObject, Forward<LambdaType>(InCallable), Forward<PayloadTypes>(InputPayload)...);
	}

	/**
	 * Static: Binds a weak object C++ lambda delegate
	 * technically this works for any functor types, but lambdas are the primary use case
	 */
	template<typename UserClass, typename LambdaType, typename... PayloadTypes>
	inline void BindWeakLambda(UserClass* InUserObject, LambdaType&& InCallable, PayloadTypes&&... InputPayload)
	{
		bIsSet = true;
		Getter.BindWeakLambda(InUserObject, Forward<LambdaType>(InCallable), Forward<PayloadTypes>(InputPayload)...);
	}

	/**
	 * Checks to see if this attribute has a 'getter' function bound
	 *
	 * @return  True if attribute is bound to a getter function
	 */
	bool IsBound() const
	{
		return Getter.IsBound();
	}

	/**
	 * Gets the attribute's 'getter' which can be bound or unbound
	 *
	 * @return  The attribute's FGetter.
	 */
	[[nodiscard]] const FGetter& GetBinding() const
	{
		return Getter;
	}

	/**
	 * Move the attribute's 'getter' or the attribute's `Value` and reset the attribute. The attribute needs to be set.
	 *
	 * @return  The attribute's FGetter or `Value`
	 */
	[[nodiscard]] TVariant<ObjectType, FGetter> Steal()
	{
		checkf(IsSet(), TEXT("It is an error to call Steal() on an unset TAttribute. Check IsSet() before calling Steal()."));
		bIsSet = false;
		if (IsBound())
		{
			return TVariant<ObjectType, FGetter>(TInPlaceType<FGetter>(), MoveTemp(Getter));
		}
		else
		{
			return TVariant<ObjectType, FGetter>(TInPlaceType<ObjectType>(), MoveTemp(Value));
		}
	}

	/**
	 * Is this attribute identical to another TAttribute.
	 *
	 * @param InOther The other attribute to compare with.
	 * @return true if the attributes are identical, false otherwise.
	 */
	bool IdenticalTo(const TAttribute& InOther) const
	{
		const bool bIsBound = IsBound();

		if ( bIsBound == InOther.IsBound() )
		{
			if ( bIsBound )
			{
				return Getter.GetHandle() == InOther.Getter.GetHandle();
			}
			else
			{
				return IsSet() == InOther.IsSet() && Value == InOther.Value;
			}
		}

		return false;
	}

private:

	/** Special explicit constructor for TAttribute::Create() */
	TAttribute( const FGetter& InGetter, bool bExplicitConstructor )
		: Value()		
		, bIsSet( true )
		, Getter(InGetter)
	{ }

	/** Special explicit constructor for TAttribute::Create() */
	TAttribute(FGetter&& InGetter, bool bExplicitConstructor)
		: Value()
		, bIsSet(true)
		, Getter(MoveTemp(InGetter))
	{ }

	// We declare ourselves as a friend (templated using OtherType) so we can access members as needed
	template< class OtherType > friend class TAttribute;

	/** Current value.  Mutable so that we can cache the value locally when using a bound Getter (allows const ref return value.) */
	mutable ObjectType Value;

	/** true when this attribute was explicitly set by a consumer, false when the attribute's value is set to the default*/
	bool bIsSet;

	/** Bound member function for this attribute (may be NULL if no function is bound.)  When set, all attempts
		to read the attribute's value will instead call this delegate to generate the value. */
	/** Our attribute's 'getter' delegate */
	FGetter Getter;
};


/**
 * Helper function for creating TAttributes from a const member function pointer, accessed through a raw pointer
 */
template<typename T, typename SourceType, typename SourceTypeOrBase, typename... PayloadTypes>
[[nodiscard]] UE_FORCEINLINE_HINT TAttribute<T> MakeAttributeRaw(const SourceType* InObject, T (SourceTypeOrBase::*InMethod)(PayloadTypes...) const, typename TDecay<PayloadTypes>::Type... InputPayload)
{
	return TAttribute<T>::Create(TAttribute<T>::FGetter::CreateRaw(InObject, InMethod, MoveTemp(InputPayload)...));
}

/**
 * Helper function for creating TAttributes from a const member function pointer, accessed through a weak pointer to the shared object
 */
template<typename T, typename SourceType, typename SourceTypeOrBase, typename... PayloadTypes>
[[nodiscard]] UE_FORCEINLINE_HINT TAttribute<T> MakeAttributeSP(const SourceType* InObject, T (SourceTypeOrBase::*InMethod)(PayloadTypes...) const, typename TDecay<PayloadTypes>::Type...  InputPayload)
{
	return TAttribute<T>::Create(TAttribute<T>::FGetter::CreateSP(InObject, InMethod, MoveTemp(InputPayload)...));
}
template<typename T, typename SourceType, typename SourceTypeOrBase, typename... PayloadTypes>
[[nodiscard]] UE_FORCEINLINE_HINT TAttribute<T> MakeAttributeSP(TSharedRef<SourceType> InObject, T (SourceTypeOrBase::*InMethod)(PayloadTypes...) const, typename TDecay<PayloadTypes>::Type...  InputPayload)
{
	return TAttribute<T>::Create(TAttribute<T>::FGetter::CreateSP(InObject, InMethod, MoveTemp(InputPayload)...));
}

template<typename T, typename SourceType, typename SourceTypeOrBase, typename... PayloadTypes>
[[nodiscard]] UE_FORCEINLINE_HINT TAttribute<T> MakeAttributeUObject(const SourceType* InObject, T (SourceTypeOrBase::*InMethod)(PayloadTypes...) const, typename TDecay<PayloadTypes>::Type... InputPayload)
{
	return TAttribute<T>::Create(TAttribute<T>::FGetter::CreateUObject(InObject, InMethod, MoveTemp(InputPayload)...));
}

/**
 * Helper function for creating TAttributes from a lambda
 * TAttribute<float> FloatAttribute = MakeAttributeLambda([]{ return 10.f; });
 */
template<typename LambdaType, typename... PayloadTypes>
[[nodiscard]] decltype(auto) MakeAttributeLambda(LambdaType&& InCallable, PayloadTypes&&... InputPayload)
{
	typedef decltype(InCallable(DeclVal<PayloadTypes>()...)) T;

	return TAttribute<T>::Create(TAttribute<T>::FGetter::CreateLambda(Forward<LambdaType>(InCallable), Forward<PayloadTypes>(InputPayload)...));
}

/**
 * Helper function for creating TAttributes from a weak shared pointer C++ lambda delegate
 */
template<typename UserClass, ESPMode Mode, typename LambdaType, typename... PayloadTypes>
[[nodiscard]] decltype(auto) MakeAttributeSPLambda(const TSharedRef<UserClass, Mode>& InUserObjectRef, LambdaType&& InCallable, PayloadTypes&&... InputPayload)
{
	typedef decltype(InCallable(DeclVal<PayloadTypes>()...)) T;

	return TAttribute<T>::Create(TAttribute<T>::FGetter::CreateSPLambda(InUserObjectRef, Forward<LambdaType>(InCallable), Forward<PayloadTypes>(InputPayload)...));
}
template <typename UserClass, typename LambdaType, typename... PayloadTypes>
[[nodiscard]] decltype(auto) MakeAttributeSPLambda(UserClass* InUserObject, LambdaType&& InCallable, PayloadTypes&&... InputPayload)
{
	typedef decltype(InCallable(DeclVal<PayloadTypes>()...)) T;

	return TAttribute<T>::Create(TAttribute<T>::FGetter::CreateSPLambda(InUserObject, Forward<LambdaType>(InCallable), Forward<PayloadTypes>(InputPayload)...));
}

/**
 * Helper function for creating TAttributes from a weak object pointer C++ lambda delegate
 */
template <typename UserClass, typename LambdaType, typename... PayloadTypes>
[[nodiscard]] decltype(auto) MakeAttributeWeakLambda(UserClass* InUserObject, LambdaType&& InCallable, PayloadTypes&&... InputPayload)
{
	typedef decltype(InCallable(DeclVal<PayloadTypes>()...)) T;

	return TAttribute<T>::Create(TAttribute<T>::FGetter::CreateWeakLambda(InUserObject, Forward<LambdaType>(InCallable), Forward<PayloadTypes>(InputPayload)...));
}
