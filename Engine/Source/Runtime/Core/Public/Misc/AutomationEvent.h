// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "UObject/CoreReflectedTypeAccessors.h"

enum class EAutomationEventType : uint8
{
	Info,
	Warning,
	Error
};

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<EAutomationEventType>();
#endif

struct FAutomationEvent
{
public:
	FAutomationEvent(EAutomationEventType InType, const FString& InMessage)
		: Type(InType)
		, Message(InMessage)
		, Context()
	{
	}

	FAutomationEvent(EAutomationEventType InType, const FString& InMessage, const FString& InContext)
		: Type(InType)
		, Message(InMessage)
		, Context(InContext)
	{
	}

public:

	EAutomationEventType Type;
	FString Message;
	FString Context;
	FGuid Artifact;
};

struct FAutomationExecutionEntry
{
	FAutomationEvent Event;
	FString Filename;
	int32 LineNumber;
	FDateTime Timestamp;

	FAutomationExecutionEntry(FAutomationEvent InEvent)
		: Event(InEvent)
		, Filename()
		, LineNumber(-1)
		, Timestamp(FDateTime::UtcNow())
	{
	}

	FAutomationExecutionEntry(FAutomationEvent InEvent, FString InFilename, int32 InLineNumber)
		: Event(InEvent)
		, Filename(InFilename)
		, LineNumber(InLineNumber)
		, Timestamp(FDateTime::UtcNow())
	{
	}

	CORE_API FString ToString() const;
	CORE_API FString ToStringFormattedEditorLog() const;
};
