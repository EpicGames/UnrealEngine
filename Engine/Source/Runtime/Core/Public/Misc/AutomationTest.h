// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Async.h"
#include "Async/Future.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Containers/StringView.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateBase.h"
#include "Delegates/DelegateInstancesImpl.h"
#include "Delegates/IDelegateInstance.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "HAL/CriticalSection.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/PlatformTime.h"
#include "HAL/PreprocessorHelpers.h"
#include "HAL/ThreadSafeBool.h"
#include "Internationalization/Regex.h"
#include "Logging/LogVerbosity.h"
#include "Math/Color.h"
#include "Misc/EnumClassFlags.h"
#include "Math/MathFwd.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AutomationEvent.h"
#include "Misc/Build.h"
#include "Misc/Char.h"
#include "Misc/CString.h"
#include "Misc/DateTime.h"
#include "Misc/FeedbackContext.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Misc/OutputDevice.h"
#include "Misc/SourceLocation.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Misc/Timespan.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

#include <atomic>

CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogLatentCommands, Log, All);
CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogAutomationTestFramework, Log, All);

class FAutomationTestBase;
class FAutomationTestTags;
class FTextFilterExpressionEvaluator;

#ifndef WITH_AUTOMATION_TESTS
	#define WITH_AUTOMATION_TESTS (WITH_DEV_AUTOMATION_TESTS || WITH_PERF_AUTOMATION_TESTS)
#endif

/** Call GetStack, with a guarantee of a non-empty return; a placeholder "Unknown",1) is used if necessary */
#define SAFE_GETSTACK(VariableName, IgnoreCount, MaxDepth)													\
	TArray<FProgramCounterSymbolInfo> VariableName = FPlatformStackWalk::GetStack(IgnoreCount, MaxDepth);	\
	if (VariableName.Num() == 0)																			\
	{																										\
		/* This is a rare failure that can occur in some circumstances */									\
		FProgramCounterSymbolInfo& Info = VariableName.Emplace_GetRef();									\
		TCString<ANSICHAR>::Strncpy(Info.Filename, "Unknown", FProgramCounterSymbolInfo::MAX_NAME_LENGTH);	\
		Info.LineNumber = 1;																				\
	}

// This macro allows for early exit of the executing unit test function when the condition is false
// It explicitly uses the condition to ensure static analysis is happy with nullptr checks
#ifndef UE_RETURN_ON_ERROR
#define UE_RETURN_ON_ERROR(Condition, Message) const bool UE_JOIN(UE____bCondition_Line_, __LINE__) = (Condition); AddErrorIfFalse(UE_JOIN(UE____bCondition_Line_, __LINE__), (Message)); if(!UE_JOIN(UE____bCondition_Line_, __LINE__)) return false
#endif

/**
* Flags for specifying automation test requirements/behavior
* Update GetTestFlagsMap when updating this enum.
*/
enum class EAutomationTestFlags
{
	None = 0x00000000,
	//~ Application context required for the test
	// Test is suitable for running within the editor
	EditorContext = 0x00000001,
	// Test is suitable for running within the client
	ClientContext = 0x00000002,
	// Test is suitable for running within the server
	ServerContext = 0x00000004,
	// Test is suitable for running within a commandlet
	CommandletContext = 0x00000008,
	// Test is suitable for running within program application (not editor, nor game)
	ProgramContext = 0x00000010,

	//~ Features required for the test - not specifying means it is valid for any feature combination
	// Test requires a non-null RHI to run correctly
	NonNullRHI = 0x00000100,
	// Test requires a user instigated session
	RequiresUser = 0x00000200,

	//~ One-off flags
	// Fast disable: turns off a test without commenting code out. Never returns for a filter.
	Disabled = 0x00010000,

	// When AutoRTFM is enabled, the test will be executed within a transaction that commits, 
	// and within another transaction that aborts. (The test will also execute non-transactionally.)
	SupportsAutoRTFM = 0x00020000,

	//~ Priority of the test
	// The highest priority possible. Showstopper/blocker.
	CriticalPriority			= 0x00100000,
	// High priority. Major feature functionality etc. 
	HighPriority				= 0x00200000,
	// Medium Priority. Minor feature functionality, major generic content issues.
	MediumPriority				= 0x00400000,
	// Low Priority. Minor content bugs. String errors. Etc.
	LowPriority					= 0x00800000,

	//~ Speed of the test
	//Super Fast Filter
	SmokeFilter					= 0x01000000,
	//Engine Level Test
	EngineFilter				= 0x02000000,
	//Product Level Test
	ProductFilter				= 0x04000000,
	//Performance Test
	PerfFilter					= 0x08000000,
	//Stress Test
	StressFilter				= 0x10000000,
	//Negative Test. For tests whose correct expected outcome is failure.
	NegativeFilter				= 0x20000000,
};

ENUM_CLASS_FLAGS(EAutomationTestFlags)

inline constexpr EAutomationTestFlags EAutomationTestFlags_ApplicationContextMask = EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::CommandletContext | EAutomationTestFlags::ProgramContext;
inline constexpr EAutomationTestFlags EAutomationTestFlags_FeatureMask            = EAutomationTestFlags::NonNullRHI | EAutomationTestFlags::RequiresUser;
inline constexpr EAutomationTestFlags EAutomationTestFlags_HighPriorityAndAbove   = EAutomationTestFlags::CriticalPriority | EAutomationTestFlags::HighPriority;
inline constexpr EAutomationTestFlags EAutomationTestFlags_MediumPriorityAndAbove = EAutomationTestFlags::CriticalPriority | EAutomationTestFlags::HighPriority | EAutomationTestFlags::MediumPriority;
inline constexpr EAutomationTestFlags EAutomationTestFlags_PriorityMask           = EAutomationTestFlags::CriticalPriority | EAutomationTestFlags::HighPriority | EAutomationTestFlags::MediumPriority | EAutomationTestFlags::LowPriority;
inline constexpr EAutomationTestFlags EAutomationTestFlags_FilterMask             = EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ProductFilter | EAutomationTestFlags::PerfFilter | EAutomationTestFlags::StressFilter | EAutomationTestFlags::NegativeFilter;

CORE_API const TMap<FString, EAutomationTestFlags>& EAutomationTestFlags_GetTestFlagsMap();

/** Flags for indicating the matching type to use for an expected message */
namespace EAutomationExpectedMessageFlags
{
	enum MatchType
	{
		// When matching expected messages, do so exactly.
		Exact,
		// When matching expected messages, just see if the message string is contained in the string to be evaluated.
		Contains,
	};

	inline const TCHAR* ToString(EAutomationExpectedMessageFlags::MatchType ThisType)
	{
		switch (ThisType)
		{
		case Contains:
			return TEXT("Contains");
		case Exact:
			return TEXT("Exact");
		}
		return TEXT("Unknown");
	}
}

/** Flags for indicating the matching type to use for an expected error message. Aliased for backwards compatibility. */
namespace EAutomationExpectedErrorFlags = EAutomationExpectedMessageFlags;

struct FAutomationTelemetryData
{
	FString DataPoint;
	double Measurement;
	FString Context;

	FAutomationTelemetryData(const FString& InDataPoint, double InMeasurement, const FString& InContext)
		:DataPoint(InDataPoint)
		, Measurement(InMeasurement)
		, Context(InContext)
	{
	}
};

/** Simple class to store the results of the execution of a automation test */
class FAutomationTestExecutionInfo
{
public:
	/** Constructor */
	FAutomationTestExecutionInfo() 
		: bSuccessful( false )
		, Duration(0.0)
		, Errors(0)
		, Warnings(0)
	{}

	/** Destructor */
	~FAutomationTestExecutionInfo()
	{
		Clear();
	}

	/** Helper method to clear out the results from a previous execution */
	CORE_API void Clear();

	CORE_API int32 RemoveAllEvents(EAutomationEventType EventType);

	CORE_API int32 RemoveAllEvents(TFunctionRef<bool(FAutomationEvent&)> FilterPredicate);

	/** Any errors that occurred during execution */
	const TArray<FAutomationExecutionEntry>& GetEntries() const { return Entries; }

	CORE_API void AddEvent(const FAutomationEvent& Event, int StackOffset = 0, bool bCaptureStack = true);

	CORE_API void AddWarning(const FString& WarningMessage);
	CORE_API void AddError(const FString& ErrorMessage);

	int32 GetWarningTotal() const { return Warnings; }
	int32 GetErrorTotal() const { return Errors; }

	const FString& GetContext() const
	{
		static FString EmptyContext;
		return ContextStack.Num() ? ContextStack.Top() : EmptyContext;
	}

	void PushContext(const FString& Context)
	{
		ContextStack.Push(Context);
	}

	void PopContext()
	{
		if ( ContextStack.Num() > 0 )
		{
			ContextStack.Pop();
		}
	}

public:

	/** Whether the automation test completed successfully or not */
	bool bSuccessful;
	
	/** Any analytics items that occurred during execution */
	TArray<FString> AnalyticsItems;

	/** Telemetry items that occurred during execution */
	TArray<FAutomationTelemetryData> TelemetryItems;

	/** Telemetry storage name set by the test */
	FString TelemetryStorage;

	/** Time to complete the task */
	double Duration;

private:
	/** Any errors that occurred during execution */
	TArray<FAutomationExecutionEntry> Entries;

	int32 Errors;
	int32 Warnings;

	TArray<FString> ContextStack;
};

/** Simple class to store the automation test info */
class FAutomationTestInfo
{
public:

	// Default constructor
	FAutomationTestInfo() = default;

	/**
	 * Constructor
	 *
	 * @param	InDisplayName - Name used in the UI
	 * @param	InFullTestPath - Dot-separated pathname to hierarchically display in the UI, which should be unique
	 * @param	InTestName - The test command string
	 * @param	InTestFlags - Test flags
	 * @param	InNumParticipantsRequired - Number of workers to run test
	 * @param	InParameterName - optional parameter. e.g. asset name
	 * @param	InSourceFile - Filesystem path to the source file which defines the test
	 * @param	InSourceFileLine - Line number in the source file which defines the test
	 * @param	InAssetPath - Filesystem path to the UAsset file which defines the test
	 * @param	InOpenCommand - An exec command to open the test
	 * @param	InTestTags - Tag metadata concatenated into one string
	 */
	FAutomationTestInfo(const FString& InDisplayName, const FString& InFullTestPath, const FString& InTestName, const EAutomationTestFlags InTestFlags, const int32 InNumParticipantsRequired, const FString& InParameterName = FString(), const FString& InSourceFile = FString(), int32 InSourceFileLine = 0, const FString& InAssetPath = FString(), const FString& InOpenCommand = FString(),  const FString& InTestTags = FString())
		: DisplayName( InDisplayName )
		, FullTestPath( InFullTestPath )
		, TestName( InTestName )
		, TestTags( InTestTags )
		, TestParameter( InParameterName )
		, SourceFile( InSourceFile )
		, SourceFileLine( InSourceFileLine )
		, AssetPath( InAssetPath )
		, OpenCommand( InOpenCommand )
		, TestFlags( InTestFlags )
		, NumParticipantsRequired( InNumParticipantsRequired )
		, NumDevicesCurrentlyRunningTest( 0 )
	{
	}

public:

	/**
	 * Add a test flag if a parent node.
	 *
	 * @Param InTestFlags - the child test flag to add.
	 */
	void AddTestFlags( const EAutomationTestFlags InTestFlags)
	{
		TestFlags |= InTestFlags;
	}

	/**
	 * Get the display name of this test.
	 *
	 * @return the display name.
	 */
	const FString& GetDisplayName() const
	{
		return DisplayName;
	}

	/**
	 * Gets the full path for this test if you wanted to run it.
	 *
	 * @return the display name.
	 */
	const FString& GetFullTestPath() const
	{
		return FullTestPath;
	}

	/**
	 * Get the test name of this test.
	 *
	 * @return The test name.
	 */
	FString GetTestName() const
	{
		return TestName;
	}

	/**
	 * Get the tags associated with this test.
	 *
	 * @return The concatenated test tags.
	 */
	FString GetTestTags() const
	{
		return TestTags;
	}

	/**
	 * Get the type of parameter. This will be the asset name for linked assets.
	 *
	 * @return the parameter.
	 */
	const FString GetTestParameter() const
	{
		return TestParameter;
	}

	/**
	 * Get the source file this test originated in.
	 *
	 * @return the source file.
	 */
	const FString GetSourceFile() const
	{
		return SourceFile;
	}

	/**
	 * Get the line number in the source file this test originated on.
	 *
	 * @return the source line number.
	 */
	const int32 GetSourceFileLine() const
	{
		return SourceFileLine;
	}

	/**
	 * Gets the asset potentially associated with the test.
	 *
	 * @return the source line number.
	 */
	const FString GetAssetPath() const
	{
		return AssetPath;
	}

	/**
	 * Gets the open command potentially associated with the test.
	 *
	 * @return the source line number.
	 */
	const FString GetOpenCommand() const
	{
		return OpenCommand;
	}

	/**
	 * Get the type of test.
	 *
	 * @return the test type.
	 */
	const EAutomationTestFlags GetTestFlags() const
	{
		return TestFlags;
	}
	
	/**
	 * Zero the number of devices running this test
	 */
	void ResetNumDevicesRunningTest()
	{
		NumDevicesCurrentlyRunningTest = 0;
	}
	
	/**
	 * Be notified of a new device running the test so we should update our flag counting these
	 */
	void InformOfNewDeviceRunningTest()
	{
		NumDevicesCurrentlyRunningTest++;
	}
	
	/**
	 * Get the number of devices running this test
	 *
	 * @return The number of devices which have been given this test to run
	 */
	const int GetNumDevicesRunningTest() const
	{
		return NumDevicesCurrentlyRunningTest;
	}

	/**
	 * Get the number of participant this test needs in order to be run
	 *
	 * @return The number of participants needed
	 */
	const int32 GetNumParticipantsRequired() const
	{
		return NumParticipantsRequired;
	}


	/**
	 * Set the display name of the child node.
	 *
	 * @Param InDisplayName - the new child test name.
	 */
	void SetDisplayName( const FString& InDisplayName )
	{
		DisplayName = InDisplayName;
	}

	/**
	 * Set the number of participant this test needs in order to be run
	 *
	 * @Param NumRequired - The new number of participants needed
	 */
	void SetNumParticipantsRequired( int32 NumRequired )
	{
		NumParticipantsRequired = NumRequired;
	}

private:
	/** Display name used in the UI */
	FString DisplayName; 

	FString FullTestPath;

	/** Test name used to run the test */
	FString TestName;

	/** Tags defined on this test */
	FString TestTags;

	/** Parameter - e.g. an asset name or map name */
	FString TestParameter;

	/** The source file this test originated in. */
	FString SourceFile;

	/** The line number in the source file this test originated on. */
	int32 SourceFileLine;

	/** The asset path associated with the test. */
	FString AssetPath;

	/** A custom open command for the test. */
	FString OpenCommand;

	/** The test flags. */
	EAutomationTestFlags TestFlags = EAutomationTestFlags::None;

	/** The number of participants this test requires */
	uint32 NumParticipantsRequired = 0;

	/** The number of devices which have been given this test to run */
	uint32 NumDevicesCurrentlyRunningTest = 0;
};


/**
 * Simple abstract base class for creating time deferred of a single test that need to be run sequentially (Loadmap & Wait, Open Editor & Wait, then execute...)
 */
class IAutomationLatentCommand : public TSharedFromThis<IAutomationLatentCommand>
{
public:
	/* virtual destructor */
	virtual ~IAutomationLatentCommand() = default;

	/**
	 * Updates the current command and will only return TRUE when it has fulfilled its role (Load map has completed and wait time has expired)
	 */
	virtual bool Update() = 0;

private:
	/**
	 * Private update that allows for use of "StartTime"
	 */
	bool InternalUpdate()
	{
		if (StartTime == 0.0)
		{
			StartTime = FPlatformTime::Seconds();
		}

		return Update();
	}

protected:
	/** Default constructor*/
	IAutomationLatentCommand()
		: StartTime(0.0f)
	{
	}

	// Gets current run time for the command for reporting purposes.
	double GetCurrentRunTime() const
	{
		if (StartTime == 0.0)
		{
			return 0.0;
		}

		return FPlatformTime::Seconds() - StartTime;
	}

	/** For timers, track the first time this ticks */
	double StartTime;

	friend class FAutomationTestFramework;
};

/**
 * A simple latent command that runs the provided function on another thread
 */
class FThreadedAutomationLatentCommand : public IAutomationLatentCommand
{
public:

	virtual ~FThreadedAutomationLatentCommand() = default;

	virtual bool Update() override
	{
		if (!Future.IsValid())
		{
			Future = Async(EAsyncExecution::Thread, MoveTemp(Function));
		}

		return Future.IsReady();
	}

	FThreadedAutomationLatentCommand(TUniqueFunction<void()> InFunction)
		: Function(MoveTemp(InFunction))
	{ }

protected:

	TUniqueFunction<void()> Function;

	TFuture<void> Future;

	friend class FAutomationTestFramework;
};


/**
 * Simple abstract base class for networked, multi-participant tests
 */
class IAutomationNetworkCommand : public TSharedFromThis<IAutomationNetworkCommand>
{
public:
	/* virtual destructor */
	virtual ~IAutomationNetworkCommand() = default;

	/** 
	 * Identifier to distinguish which worker on the network should execute this command 
	 * 
	 * The index of the worker that should execute this command
	 */
	virtual uint32 GetRoleIndex() const = 0;
	
	/** Runs the network command */
	virtual void Run() = 0;
};

struct FAutomationExpectedMessage
{
	// Original string pattern matching expected log message.
	// If IsRegex is false, it is this string that is used to match.
	// Otherwise MessagePatternRegex is set to a valid pointer of FRegexPattern.
	// The base pattern string is preserved none the less to allow checks for duplicate entries.
	FString MessagePatternString;
	// Regular expression pattern from MessagePatternString if regex option was true(default), otherwise it is not set.
	TOptional<FRegexPattern> MessagePatternRegex;
	// Type of comparison to perform on error log using MessagePattern.
	EAutomationExpectedMessageFlags::MatchType CompareType;
	/** 
	 * Number of occurrences expected for message. If set greater than 0, it will cause the test to fail if the
	 * exact number of occurrences expected is not matched. If set to 0, it will cause the test to fail if
	 * there were no occurrences at all. If negative, it will suppress all matching messages.
	 */
	int32 ExpectedNumberOfOccurrences;
	int32 ActualNumberOfOccurrences;
	// Log message Verbosity
	ELogVerbosity::Type Verbosity;

	/**
	* Constructor
	*/
	FAutomationExpectedMessage(FString& InMessagePattern, ELogVerbosity::Type InVerbosity, EAutomationExpectedMessageFlags::MatchType InCompareType, int32 InExpectedNumberOfOccurrences = 1, bool IsRegex = true)
		: MessagePatternString(InMessagePattern)
		, CompareType(InCompareType)
		, ExpectedNumberOfOccurrences(InExpectedNumberOfOccurrences)
		, ActualNumberOfOccurrences(0)
		, Verbosity(InVerbosity)
	{
		if (IsRegex)
		{
			MessagePatternRegex = FRegexPattern((InCompareType == EAutomationExpectedMessageFlags::Exact) ? FString::Printf(TEXT("^%s$"), *InMessagePattern) : InMessagePattern, ERegexPatternFlags::CaseInsensitive);
		}		
	}

	FAutomationExpectedMessage(FString& InMessagePattern, ELogVerbosity::Type InVerbosity, int32 InExpectedNumberOfOccurrences)
		: MessagePatternString(InMessagePattern)
		, MessagePatternRegex(FRegexPattern(InMessagePattern, ERegexPatternFlags::CaseInsensitive))
		, CompareType(EAutomationExpectedMessageFlags::Contains)
		, ExpectedNumberOfOccurrences(InExpectedNumberOfOccurrences)
		, ActualNumberOfOccurrences(0)
		, Verbosity(InVerbosity)
	{}

	inline bool IsRegex() const
	{
		return MessagePatternRegex.IsSet();
	}

	inline bool IsExactCompareType() const
	{
		return CompareType == EAutomationExpectedMessageFlags::Exact;
	}

	/// <summary>
	/// Look if Message matches the expected message and increment internal counter if true.
	/// </summary>
	/// <param name="Message"></param>
	/// <returns></returns>
	bool Matches(const FString& Message)
	{
		bool HasMatch = false;
		if (IsRegex())
		{
			FRegexMatcher MessageMatcher(MessagePatternRegex.GetValue(), Message);
			HasMatch = MessageMatcher.FindNext();
		}
		else
		{
			HasMatch = Message.Contains(MessagePatternString) && (!IsExactCompareType() || Message.Len() == MessagePatternString.Len());
		}
		ActualNumberOfOccurrences += HasMatch;
		return HasMatch;
	}

	bool operator==(const FAutomationExpectedMessage& Other) const
	{
		return MessagePatternString == Other.MessagePatternString;
	}

	bool operator<(const FAutomationExpectedMessage& Other) const
	{
		return MessagePatternString < Other.MessagePatternString;
	}

};

UE_FORCEINLINE_HINT uint32 GetTypeHash(const FAutomationExpectedMessage& Object)
{
	return GetTypeHash(Object.MessagePatternString);
}

struct FAutomationScreenshotData
{
	FString ScreenShotName;
	FString VariantName;
	FString Context;
	FString TestName;
	FString Notes;

	FGuid Id;
	FString Commit;

	int32 Width;
	int32 Height;

	// RHI Details
	FString Platform;
	FString Rhi;
	FString FeatureLevel;
	bool bIsStereo;

	// Track if ScreenShotData was created with Substrate enabled or not
	bool bIsSubstrate;

	// Hardware Details
	FString Vendor;
	FString AdapterName;
	FString AdapterInternalDriverVersion;
	FString AdapterUserDriverVersion;
	FString UniqueDeviceId;

	// Quality Levels
	float ResolutionQuality;
	int32 ViewDistanceQuality;
	int32 AntiAliasingQuality;
	int32 ShadowQuality;
	int32 GlobalIlluminationQuality;
	int32 ReflectionQuality;
	int32 PostProcessQuality;
	int32 TextureQuality;
	int32 EffectsQuality;
	int32 FoliageQuality;
	int32 ShadingQuality;

	// Comparison Requests
	bool bHasComparisonRules;
	uint8 ToleranceRed;
	uint8 ToleranceGreen;
	uint8 ToleranceBlue;
	uint8 ToleranceAlpha;
	uint8 ToleranceMinBrightness;
	uint8 ToleranceMaxBrightness;
	float MaximumLocalError;
	float MaximumGlobalError;
	bool bIgnoreAntiAliasing;
	bool bIgnoreColors;

	// Path of the screenshot generated from AutomationCommon::GetScreenShotPath()
	FString ScreenshotPath;

	FAutomationScreenshotData()
		: Id()
		, Commit()
		, Width(0)
		, Height(0)
		, bIsStereo(false)
		, bIsSubstrate(false)
		, ResolutionQuality(1.0f)
		, ViewDistanceQuality(0)
		, AntiAliasingQuality(0)
		, ShadowQuality(0)
		, GlobalIlluminationQuality(0)
		, ReflectionQuality(0)
		, PostProcessQuality(0)
		, TextureQuality(0)
		, EffectsQuality(0)
		, FoliageQuality(0)
		, ShadingQuality(0)
		, bHasComparisonRules(false)
		, ToleranceRed(0)
		, ToleranceGreen(0)
		, ToleranceBlue(0)
		, ToleranceAlpha(0)
		, ToleranceMinBrightness(0)
		, ToleranceMaxBrightness(255)
		, MaximumLocalError(0.0f)
		, MaximumGlobalError(0.0f)
		, bIgnoreAntiAliasing(false)
		, bIgnoreColors(false)
	{
	}
};

struct FAutomationScreenshotCompareResults
{
	FGuid UniqueId;
	FString ErrorMessage;
	double MaxLocalDifference = 0.0;
	double GlobalDifference = 0.0;
	bool bWasNew = false;
	bool bWasSimilar = false;
	FString IncomingFilePath;
	FString ReportComparisonFilePath;
	FString ReportApprovedFilePath;
	FString ReportIncomingFilePath;
	FString ScreenshotPath;

	FAutomationScreenshotCompareResults()
		: UniqueId()
		, MaxLocalDifference(0.0)
		, GlobalDifference(0.0)
		, bWasNew(false)
		, bWasSimilar(false)
	{ }

	FAutomationScreenshotCompareResults(
		FGuid InUniqueId,
		FString InErrorMessage,
		double InMaxLocalDifference,
		double InGlobalDifference,
		bool InWasNew,
		bool InWasSimilar,
		FString InIncomingFilePath,
		FString InReportComparisonFilePath,
		FString InReportApprovedFilePath,
		FString InReportIncomingFilePath,
		FString InScreenshotPath
	)
		: UniqueId(InUniqueId)
		, ErrorMessage(InErrorMessage)
		, MaxLocalDifference(InMaxLocalDifference)
		, GlobalDifference(InGlobalDifference)
		, bWasNew(InWasNew)
		, bWasSimilar(InWasSimilar)
		, IncomingFilePath(InIncomingFilePath)
		, ReportComparisonFilePath(InReportComparisonFilePath)
		, ReportApprovedFilePath(InReportApprovedFilePath)
		, ReportIncomingFilePath(InReportIncomingFilePath)
		, ScreenshotPath(InScreenshotPath)
	{ }

	CORE_API FAutomationEvent ToAutomationEvent() const;
};

enum class EAutomationComparisonToleranceLevel : uint8
{
	Zero,
	Low,
	Medium,
	High
};

struct FAutomationComparisonToleranceAmount
{
public:

	FAutomationComparisonToleranceAmount()
		: Red(0)
		, Green(0)
		, Blue(0)
		, Alpha(0)
		, MinBrightness(0)
		, MaxBrightness(255)
	{
	}

	FAutomationComparisonToleranceAmount(uint8 R, uint8 G, uint8 B, uint8 A, uint8 InMinBrightness, uint8 InMaxBrightness)
		: Red(R)
		, Green(G)
		, Blue(B)
		, Alpha(A)
		, MinBrightness(InMinBrightness)
		, MaxBrightness(InMaxBrightness)
	{
	}

	static FAutomationComparisonToleranceAmount FromToleranceLevel(EAutomationComparisonToleranceLevel InTolerance)
	{
		switch (InTolerance)
		{
		case EAutomationComparisonToleranceLevel::Low:
			return FAutomationComparisonToleranceAmount(16, 16, 16, 16, 16, 240);
		case EAutomationComparisonToleranceLevel::Medium:
			return FAutomationComparisonToleranceAmount(24, 24, 24, 24, 24, 220);
		case EAutomationComparisonToleranceLevel::High:
			return FAutomationComparisonToleranceAmount(32, 32, 32, 32, 64, 96);
		}
		// Zero
		return FAutomationComparisonToleranceAmount(0, 0, 0, 0, 0, 255);
	}

	uint8 Red;
	uint8 Green;
	uint8 Blue;
	uint8 Alpha;
	uint8 MinBrightness;
	uint8 MaxBrightness;
};

/**
 * Delegate type for when a test screenshot has been captured
 *
 * The first parameter is the array of the raw color data.
 * The second parameter is the image metadata.
 */
DECLARE_DELEGATE_TwoParams(FOnTestScreenshotCaptured, const TArray<FColor>&, const FAutomationScreenshotData&);

DECLARE_DELEGATE_ThreeParams(FOnTestScreenshotAndTraceCaptured, const TArray<FColor>&, const TArray<uint8>&, const FAutomationScreenshotData&);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnTestScreenshotComparisonComplete, const FAutomationScreenshotCompareResults& /*CompareResults*/);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnTestScreenshotComparisonReport, const FAutomationScreenshotCompareResults& /*CompareResults*/);

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTestDataRetrieved, bool /*bWasNew*/, const FString& /*JsonData*/);

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPerformanceDataRetrieved, bool /*bSuccess*/, const FString& /*ErrorMessage*/);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnTestEvent, FAutomationTestBase*);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnTestSectionEvent, const FString& /*Section*/);

DECLARE_DELEGATE_OneParam(FBeforeTagMappingConfigSaved, const FString& /*ConfigPath*/);

DECLARE_DELEGATE_OneParam(FAfterTagMappingConfigSaved, const FString& /*ConfigPath*/);


/** Class representing the main framework for running automation tests */
class FAutomationTestFramework
{
public:
	/** Called right before automated test is about to begin */
	FSimpleMulticastDelegate PreTestingEvent;

	/** Called after all automated tests have completed */
	FSimpleMulticastDelegate PostTestingEvent;

	/** Called when each automated test is starting */
	FOnTestEvent OnTestStartEvent;

	/** Called when each automated test is ending */
	FOnTestEvent OnTestEndEvent;

	/** Called when a screenshot comparison completes. */
	FOnTestScreenshotComparisonComplete OnScreenshotCompared;

	/** Called when a screenshot comparison result is reported */
	FOnTestScreenshotComparisonReport OnScreenshotComparisonReport;

	/** Called when the test data is retrieved. */
	FOnTestDataRetrieved OnTestDataRetrieved;

	/** Called when the performance data is retrieved. */
	FOnPerformanceDataRetrieved OnPerformanceDataRetrieved;

	/** The final call related to screenshots, after they've been taken, and after they've been compared (or not if automation isn't running). */
	FSimpleMulticastDelegate OnScreenshotTakenAndCompared;

	/** Called before all chosen tests run. */
	FSimpleMulticastDelegate OnBeforeAllTestsEvent;

	/** Called after all chosen tests run have finished. */
	FSimpleMulticastDelegate OnAfterAllTestsEvent;

	/** Called entering test section. */
	CORE_API FOnTestSectionEvent& GetOnEnteringTestSection(const FString& Section);
	CORE_API void TriggerOnEnteringTestSection(const FString& Section) const;
	CORE_API bool IsAnyOnEnteringTestSectionBound() const;

	/** Called leaving test section. */
	CORE_API FOnTestSectionEvent& GetOnLeavingTestSection(const FString& Section);
	CORE_API void TriggerOnLeavingTestSection(const FString& Section) const;
	CORE_API bool IsAnyOnLeavingTestSectionBound() const;

	/**
	 * Return the singleton instance of the framework.
	 *
	 * @return The singleton instance of the framework.
	 */
	static CORE_API FAutomationTestFramework& Get();
	static FAutomationTestFramework& GetInstance() { return Get(); }

	/**
	 * Gets a scratch space location outside of the project and saved directories.  When an automation test needs
	 * to do something like generate project files, or create new projects it should use this directory, rather
	 * than pollute other areas of the machine.
	 */
	CORE_API FString GetUserAutomationDirectory() const;

	/**
	 * Register a automation test into the framework. The automation test may or may not be necessarily valid
	 * for the particular application configuration, but that will be determined when tests are attempted
	 * to be run.
	 *
	 * @param	InTestNameToRegister	Name of the test being registered
	 * @param	InTestToRegister		Actual test to register
	 *
	 * @return	true if the test was successfully registered; false if a test was already registered under the same
	 *			name as before
	 */
	CORE_API bool RegisterAutomationTest( const FString& InTestNameToRegister, FAutomationTestBase* InTestToRegister );

	/**
	 * Unregister a automation test with the provided name from the framework.
	 *
	 * @return true if the test was successfully unregistered; false if a test with that name was not found in the framework.
	 */
	CORE_API bool UnregisterAutomationTest( const FString& InTestNameToUnregister );

	/**
	 * Register tag metadata for a test into the framework.
	 * 
	 * @param	InTestNameToRegister	FullName of the test to associate with tags
	 * @param	InTestTagsToRegister	Concatenated string of tags to register
	 * @param   InImmutable				Whether tags are immutable. If false, merge with existing tags, keeping the immutable
	 * 
	 * @return	true if tags for the test successfully registered or overwritten; false if tags were already registered for the test
	 */
	CORE_API bool RegisterAutomationTestTags(const FString& InTestNameToRegister, const FString& InTestTagsToRegister, bool InImmutable = true);

	/**
	 * Unregister tags for a automation test from the framework.
	 * 
	 * @param	InTestNameToUnregister	FullName of the test to remove tags from
	 * 
	 * @return true if test tags were successfully unregistered; false if tags for a test with that name was not found in the framework.
	 */
	CORE_API bool UnregisterAutomationTestTags(const FString& InTestNameToUnregister);

	/**
	 * Helper to register tags for a individual test enumerated by a Complex test
	 *
	 * @param	InTest					The Complex test to associate with
	 * @param	InBeautifiedTestName	The name of the individual test
	 * @param	InTestTagsToRegister	Concatenated string of tags to register
	 *
	 * @return true if test tags were successfully unregistered; false if tags for a test with that name was not found in the framework.
	 */
	CORE_API bool RegisterComplexAutomationTestTags(const FAutomationTestBase* InTest, const FString& InBeautifiedTestName, const FString& InTestTagsToRegister);

	CORE_API TSet<FString> GetAllExistingTags();

	CORE_API bool IsTagImmutable(const FString& InTestName, const FString& InTag) const;

	/**
	 * Fetch the tags associated with a test
	 * 
	 * @param	InTestName	Name of the test to find tags for
	 * 
	 * @return	a string of all concatenated tags; empty string if test is not registered
	 */
	CORE_API FString GetTagsForAutomationTest(const FString& InTestName);

	/**
	 * Fetch the tags associated with a test as an array of strings
	 *
	 * @param	InTestName	Name of the test to find tags for
	 *
	 * @return	an array of tags; empty if test is not registered
	 */
	CORE_API TArray<FString> GetTagsForAutomationTestAsArray(const FString& InTestName);

	/**
	 * Enqueues a latent command for execution on a subsequent frame
	 *
	 * @param NewCommand - The new command to enqueue for deferred execution
	 */
	CORE_API void EnqueueLatentCommand(TSharedPtr<IAutomationLatentCommand> NewCommand);

	/**
	 * Enqueues a network command for execution in accordance with this workers role
	 *
	 * @param NewCommand - The new command to enqueue for network execution
	 */
	CORE_API void EnqueueNetworkCommand(TSharedPtr<IAutomationNetworkCommand> NewCommand);

	/**
	 * Checks if a provided test is contained within the framework.
	 *
	 * @param InTestName	Name of the test to check
	 *
	 * @return	true if the provided test is within the framework; false otherwise
	 */
	CORE_API bool ContainsTest( const FString& InTestName ) const;
		
	/**
	 * Attempt to run all fast smoke tests that are valid for the current application configuration.
	 *
	 * @return	true if all smoke tests run were successful, false if any failed
	 */
	CORE_API bool RunSmokeTests();

	/**
	 * Reset status of worker (delete local files, etc)
	 */
	CORE_API void ResetTests();

	/**
	 * Attempt to start the specified test.
	 *
	 * @param	InTestToRun			Name of the test that should be run
	 * @param	InRoleIndex			Identifier for which worker in this group that should execute a command
	 * @param	InFullTestPath		Full test path
	 */
	CORE_API void StartTestByName( const FString& InTestToRun, const int32 InRoleIndex, const FString& InFullTestPath = FString() );

	/**
	 * Stop the current test and return the results of execution
	 *
	 * @return	true if the test ran successfully, false if it did not (or the test could not be found/was invalid)
	 */
	CORE_API bool StopTest( FAutomationTestExecutionInfo& OutExecutionInfo );

	/**
	 * Execute all latent functions that complete during update
	 *
	 * @return - true if the latent command queue is now empty and the test is complete
	 */
	CORE_API bool ExecuteLatentCommands();

	/**
	 * Execute the next network command if you match the role, otherwise just dequeue
	 *
	 * @return - true if any network commands were in the queue to give subsequent latent commands a chance to execute next frame
	 */
	CORE_API bool ExecuteNetworkCommands();

	/**
	 * Dequeue all latent and network commands
	 */
	CORE_API void DequeueAllCommands();

	/**
	 * Whether there is no latent command in queue
	 */
	bool IsLatentCommandQueueEmpty() const
	{
		return LatentCommands.IsEmpty();
	}

	/**
	 * Load any modules that are not loaded by default and have test classes in them
	 */
	CORE_API void LoadTestModules();

	/**
	 * Populates the provided array with the names of all tests in the framework that are valid to run for the current
	 * application settings.
	 *
	 * @param	TestInfo	Array to populate with the test information
	 */
	CORE_API void GetValidTestNames( TArray<FAutomationTestInfo>& TestInfo ) const;

	/**
	 * Collects all registered tests with associated tags matching the pattern
	 * 
	 * @param	OutTestNames	Array that will be modified to contain only the set of matching tests
	 * @param	TagPattern		An "Advanced Search Syntax" query to select matching tags from the registered tests
	 * 
	 * @see FTextFilterExpressionEvaluator
	 * @see FAutomationTestBase::GetTestFullName
	 */
	CORE_API void GetTestFullNamesMatchingTagPattern(TArray<FString>& OutTestNames, const FString& TagPattern) const;

	CORE_API void GetPossibleRestrictedPaths(const FString& BasePath, const TArray<FString>& RestrictedFolderNames, TArray<FString>& OutRestrictedFolders) const;

	/**
	 * Reads test tag mappings from all Tags.ini config files
	 */
	CORE_API void LoadTestTagMappings();

	/**
	 * Reads test tag mappings under a certain base path
	 */
	CORE_API void LoadTestTagMappings(const FString& BasePath, const bool RestrictedPathsOnly);


	/**
	 * Saves test tag mappings optionally invoking before / after events.
	 */
	CORE_API void SaveTestTagMappings(const TArray<FString>& TestFullNames, const TArray<FString>& TestFilePaths, const FBeforeTagMappingConfigSaved& BeforeConfigSaved, const FAfterTagMappingConfigSaved& AfterConfigSaved) const;

	/**
	 * Whether the testing framework should allow content to be tested or not.  Intended to block developer directories.
	 * @param Path - Full path to the content in question
	 * @return - Whether this content should have tests performed on it
	 */
	CORE_API bool ShouldTestContent(const FString& Path) const;

	/**
	 * Sets whether we want to include content in developer directories in automation testing
	 */
	CORE_API void SetDeveloperDirectoryIncluded(const bool bInDeveloperDirectoryIncluded);

	/**
	* Sets which set of tests to pull from.
	*/
	CORE_API void SetRequestedTestFilter(const EAutomationTestFlags InRequestedTestFlags);
	

	/**
	 * Accessor for delegate called when a png screenshot is captured 
	 */
	CORE_API FOnTestScreenshotCaptured& OnScreenshotCaptured();

	/**
	 * Accessor for delegate called when a png screenshot is captured and a frame trace
	 */
	CORE_API FOnTestScreenshotAndTraceCaptured& OnScreenshotAndTraceCaptured();

	/**
	 * Sets forcing smoke tests.
	 */
	void SetForceSmokeTests(const bool bInForceSmokeTests)
	{
		bForceSmokeTests = bInForceSmokeTests;
	}

	bool GetCaptureStack() const
	{
		return bCaptureStack && !NeedSkipStackWalk();
	}

	/**
	 * Used to disabled stack capture when an error or warning event is triggered.
	 * Setting bCapture=true does not guarantees GetCaptureStack()=true because that method also depend on NeedSkipStackWalk(). 
	 */
	void SetCaptureStack(bool bCapture)
	{
		bCaptureStack = bCapture;
	}

	/**
	 * Adds a analytics string to the current test to be parsed later.  Must be called only when an automation test is in progress
	 *
	 * @param	AnalyticsItem	Log item to add to the current test
	 */
	CORE_API void AddAnalyticsItemToCurrentTest( const FString& AnalyticsItem );

	/**
	 * Returns the actively executing test or null if there isn't one
	 */
	FAutomationTestBase* GetCurrentTest() const
	{
		return CurrentTest;
	}

	/**
	 * Returns the actively executing test full path
	 */
	FString GetCurrentTestFullPath() const
	{
		return CurrentTestFullPath;
	}

	/**
	 * Whether to skip stack walk while iterating for listing the tests
	 */
	static CORE_API bool NeedSkipStackWalk();

	/**
	 * Whether to output blueprint functional test metadata to the log when test is running
	 */
	static CORE_API bool NeedLogBPTestMetadata();

	/**
	 * Whether to also run stereo test variants for screenshot functional tests
	 */
	static CORE_API bool NeedPerformStereoTestVariants();

	/**
	 * Whether to skip variants when the baseline test fails, and skip saving screenshots for successful variants
	 */
	static CORE_API bool NeedUseLightweightStereoTestVariants();

	/**
	 * Notify that the screenshot comparison has completed
	 */
	CORE_API void NotifyScreenshotComparisonComplete(const FAutomationScreenshotCompareResults& CompareResults);

	/**
	 * Notify the screenshot comparison report to the framework
	 */
	CORE_API void NotifyScreenshotComparisonReport(const FAutomationScreenshotCompareResults& CompareResults);

	CORE_API void NotifyTestDataRetrieved(bool bWasNew, const FString& JsonData);
	CORE_API void NotifyPerformanceDataRetrieved(bool bSuccess, const FString& ErrorMessage);

	CORE_API void NotifyScreenshotTakenAndCompared();

	/**
	 * Internal helper method designed to check if the given test is able to run in the current environment.
	 *
	 * @param	InTestToRun test name
	 * @param	OutReason the related reason of the skipping
	 * @param	OutWarn the related warning of the skipping
	 *
	 * @return	true if the test is able to run; false if it is unable to run.
	 */
	CORE_API bool CanRunTestInEnvironment(const FString& InTestToRun, FString* OutReason, bool* OutWarn) const;

private:

	/** Special output device used during automation testing to gather messages that happen during tests */
	 class FAutomationTestOutputDevice : public FOutputDevice
	{
	public:
		FAutomationTestOutputDevice() 
			: CurTest( nullptr ) {}

		~FAutomationTestOutputDevice()
		{
			CurTest = nullptr;
		}

		/**
		 * FOutputDevice interface
		 *
		 * @param	V		String to serialize within the output device
		 * @param	Event	Event associated with the string
		 */
		virtual void Serialize( const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category ) override;

		/**
		 * FOutputDevice interface
		 *
		 * Make it unbuffered by returning true
		 */
		virtual bool CanBeUsedOnMultipleThreads() const override
		{
			return true;
		}

		/**
		 * Set the automation test associated with the output device. The automation test is where all warnings, errors, etc.
		 * will be routed to.
		 *
		 * @param	InAutomationTest	Automation test to associate with the output device.
		 */
		void SetCurrentAutomationTest( FAutomationTestBase* InAutomationTest )
		{
			CurTest = InAutomationTest;
		}

	private:
		/** Associated automation test; all warnings, errors, etc. are routed to the automation test to track */
		std::atomic<FAutomationTestBase*>CurTest;
	};

	 /** Special feedback context used during automated testing to filter messages that happen during tests */
	 class FAutomationTestMessageFilter: public FFeedbackContext
	 {
	 public:
		 FAutomationTestMessageFilter()
			: CurTest(nullptr)
			, DestinationContext(nullptr) {}

		 ~FAutomationTestMessageFilter()
		 {
			 DestinationContext = nullptr;
			 CurTest = nullptr;
		 }

		 /**
		  * FOutputDevice interface
		  *
		  * @param	V		String to serialize within the context
		  * @param	Event	Event associated with the string
		  */
		 virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override;
		 virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time) override;

		 virtual void SerializeRecord(const UE::FLogRecord& Record) override;

		 /**
		  * FOutputDevice interface
		  *
		  * Make it unbuffered by returning true
		  */
		 virtual bool CanBeUsedOnMultipleThreads() const override
		 {
			 return true;
		 }

		 /**
		  * Set the automation test associated with the feedback context. The automation test is what will be used
		  * to determine if a given warning or error is expected and thus should not be treated as a warning or error
		  * by the destination context.
		  *
		  * @param	InAutomationTest	Automation test to associate with the feedback context.
		  */
		 void SetCurrentAutomationTest(FAutomationTestBase* InAutomationTest)
		 {
			 CurTest = InAutomationTest;
		 }

		 /**
		  * Set the destination associated with the feedback context. The automation test is where all warnings, errors, etc.
		  * will be routed to.
		  *
		  * @param	InAutomationTest	Automation test to associate with the feedback context.
		  */
		 void SetDestinationContext(FFeedbackContext* InDestinationContext)
		 {
			 DestinationContext = InDestinationContext;
		 }

	 private:
		 std::atomic<FAutomationTestBase*> CurTest;
		 std::atomic<FFeedbackContext*> DestinationContext;
		 FTransactionallySafeCriticalSection ActionCS;
	 };

	friend class FAutomationTestOutputDevice;
	/** Helper method called to prepare settings for automation testing to follow */
	CORE_API void PrepForAutomationTests();

	/** Helper method called after automation testing is complete to restore settings to how they should be */
	CORE_API void ConcludeAutomationTests();

	/**
	 * Helper method to dump the contents of the provided test name to execution info map to the provided feedback context
	 *
	 * @param	InContext		Context to dump the execution info to
	 * @param	InInfoToDump	Execution info that should be dumped to the provided feedback context
	 */
	CORE_API void DumpAutomationTestExecutionInfo( const TMap<FString, FAutomationTestExecutionInfo>& InInfoToDump );

	/**
	 * Internal helper method designed to simply start the provided test name.
	 *
	 * @param	InTestToRun			Name of the test that should be run
	 * @param	InFullTestPath		Full test path
	 */
	CORE_API void InternalStartTest( const FString& InTestToRun, const FString& InFullTestPath );

	/**
	 * Internal helper method designed to stop current executing test and return the results of execution.
	 *
	 * @return	true if the test was successfully run; false if it was not, could not be found, or is invalid for
	 *			the current application settings
	 */
	CORE_API bool InternalStopTest(FAutomationTestExecutionInfo& OutExecutionInfo);

	/** Constructor */
	CORE_API FAutomationTestFramework();

	/** Destructor */
	CORE_API ~FAutomationTestFramework();

	// Copy constructor and assignment operator intentionally left unimplemented
	CORE_API FAutomationTestFramework( const FAutomationTestFramework& );
	CORE_API FAutomationTestFramework& operator=( const FAutomationTestFramework& );

	/** Specialized output device used for automation testing */
	FAutomationTestOutputDevice AutomationTestOutputDevice;

	/** Specialized feedback context used for message filtering during automated testing */
	FAutomationTestMessageFilter AutomationTestMessageFilter;

	FFeedbackContext* OriginalGWarn = nullptr;

	/** Mapping of automation test names to their respective object instances */
	TMap<FString, FAutomationTestBase*> AutomationTestClassNameToInstanceMap;

	/* Set of all tags whether they are code declared or through config file */
	TSet<FString> AllExistingTags;

	/**
	 * Mapping of full test names to their registered tags.
	 */
	TMap<FString, FString> TestFullNameToTagDataMap;

	/**
	 * Map of immutable tags for each test. Code defined tags are immutable.
	 */
	TMap<FString, TSet<FString>> ImmutableTags;

	/** Queue of deferred commands */
	TQueue<TSharedPtr<IAutomationLatentCommand>, EQueueMode::SingleThreaded> LatentCommands;

	/** Queue of deferred commands */
	TQueue<TSharedPtr<IAutomationNetworkCommand>, EQueueMode::SingleThreaded> NetworkCommands;

	/** Whether we are currently executing smoke tests for startup/commandlet to minimize log spam */
	EAutomationTestFlags RequestedTestFilter;

	/** Time when the test began executing */
	double StartTime;

	/** True if the execution of the test (but possibly not the latent actions) were successful */
	bool bTestSuccessful;

	/** Pointer to the current test being run */
	FAutomationTestBase* CurrentTest;

	/** Copy of the parameters for the active test */
	FString Parameters;

	/** Full test path as given by the automation controller of the active test */
	FString CurrentTestFullPath;

	/** Whether we want to run automation tests on content within the Developer Directories */
	bool bDeveloperDirectoryIncluded;

	/** Participation role as given by the automation controller */
	uint32 NetworkRoleIndex;

	/** Delegate called at the end of the frame when a screenshot is captured and a .png is requested */
	FOnTestScreenshotCaptured TestScreenshotCapturedDelegate;

	/** Delegate called at the end of the frame when a screenshot and frame trace is captured and a .png is requested */
	FOnTestScreenshotAndTraceCaptured TestScreenshotAndTraceCapturedDelegate;

	/** Forces running smoke tests */
	bool bForceSmokeTests;

	bool bCaptureStack;

	TMap<FString, FOnTestSectionEvent> OnEnteringTestSectionEvent;
	TMap<FString, FOnTestSectionEvent> OnLeavingTestSectionEvent;

	/**
	 * Evaluate whether a tag string matches the pattern string
	 * 
	 * @param	Tags		Concatenated string of all tags defined for a test
	 * @param	TagPattern	A query pattern
	 * @see FTextFilterExpressionEvaluator
	 */
	bool TagsMatchPattern(const FString& Tags, const FString& TagPattern) const;

	/** Cached filter context */
	TSharedPtr<FTextFilterExpressionEvaluator> TagFilter;
};

/** Wrapper class to simplify tag registration */
class FAutomationTestTags
{
public:
	/**
	 * Register tags for test by its fully defined name, which will unregister when this object leaves scope
	 * 
	 * @param	InTestFullName	The unique path-name identifying this test
	 * @param	InTags			A string defining all concatenated tags to register
	 * @see FAutomationTestBase::GetTestFullName
	 */
	FAutomationTestTags(const FString& InTestFullName, const FString& InTags)
	{
		LLM_SCOPE_BYNAME(TEXT("AutomationTest/Framework"));
		TestFullName = InTestFullName;
		bool Registered = FAutomationTestFramework::Get().RegisterAutomationTestTags( InTestFullName, InTags );
		if (!Registered)
		{
			UE_LOGF(LogAutomationTestFramework, Warning, "Tag registration already exists for test '%ls', existing value was not overridden", *TestFullName);
		}
	}

	/**
	 * No-op default ctor performs no registration
	 */
	FAutomationTestTags() {}

	~FAutomationTestTags()
	{
		if (!TestFullName.IsEmpty())
		{
			//ignore failure to unregister due to already being unregistered
			FAutomationTestFramework::Get().UnregisterAutomationTestTags(TestFullName);
		}
	}
private:
	FString TestFullName;
};

/** Simple abstract base class for all automation tests */
class FAutomationTestBase
{
public:
	/**
	 * Constructor
	 *
	 * @param	InName	Name of the test
	 */
	CORE_API FAutomationTestBase( const FString& InName, const bool bInComplexTask );

	/** Destructor */
	CORE_API virtual ~FAutomationTestBase() ;

	/** Log flags */
	static CORE_API bool bSuppressLogWarnings;
	static CORE_API bool bSuppressLogErrors;
	static CORE_API bool bElevateLogWarningsToErrors;
	static CORE_API TArray<FString> SuppressedLogCategories;

	/**
	 * Pure virtual method; returns the flags associated with the given automation test
	 *
	 * @return	Automation test flags associated with the test
	 */
	virtual EAutomationTestFlags GetTestFlags() const = 0;

	/** Gets the C++ name of the test. */
	FString GetTestName() const { return TestName; }

	/** Gets the parameter context of the test. */
	FString GetTestContext() const { return TestParameterContext; }

	/**
	* Returns the beautified test name
	*/
	virtual FString GetBeautifiedTestName() const = 0;

	/**
	 * Returns the beautified test name with test context. Should return what is displayed in the Test Automation UI. See GenerateTestNames()
	 */
	virtual FString GetTestFullName() const {
		if (FAutomationTestFramework::Get().GetCurrentTest() == this)
		{
			return FAutomationTestFramework::Get().GetCurrentTestFullPath();
		}
		if (GetTestContext().IsEmpty()) { return GetBeautifiedTestName(); }
		return FString::Printf(TEXT("%s.%s"), *GetBeautifiedTestName(), *GetTestContext());
	}

	/**
	 * Pure virtual method; returns the number of participants for this test
	 *
	 * @return	Number of required participants
	 */
	virtual uint32 GetRequiredDeviceNum() const = 0;

	/** Clear any execution info/results from a prior running of this test */
	CORE_API void ClearExecutionInfo();

	/**
	 * Adds an error message to this test
	 *
	 * @param	InError	Error message to add to this test
	 */
	CORE_API virtual void AddError( const FString& InError, int32 StackOffset = 0 );

	/**
	 * Adds an error message to this test if the condition is false
	 *
	 * @param   bCondition The condition to validate.
	 * @param   InError	   Error message to add to this test
	 * @return	False if there was an error
	 */
	CORE_API virtual bool AddErrorIfFalse( bool bCondition, const FString& InError, int32 StackOffset = 0 );

	/**
	 * Adds a warning to this test
	 *
	 * @param	InWarning	Warning message to add to this test
	 */
	CORE_API virtual void AddWarning( const FString& InWarning, int32 StackOffset = 0);

	/**
	 * Adds a log item to this test
	 *
	 * @param	InLogItem	Log item to add to this test
	 */
	CORE_API virtual void AddInfo( const FString& InLogItem, int32 StackOffset = 0, bool bCaptureStack = false);

	/**
	 * Adds an automation event directly into the execution log.
	 *
	 * @param	InLogItem	Log item to add to this test
	 */
	CORE_API virtual void AddEvent(const FAutomationEvent& InEvent, int32 StackOffset = 0, bool bCaptureStack = false);

	/**
	 * Adds a analytics string to parse later
	 *
	 * @param	InLogItem	Log item to add to this test
	 */
	CORE_API virtual void AddAnalyticsItem(const FString& InAnalyticsItem);

	/**
	 * Adds a telemetry data point measurement
	 *
	 * @param	DataPoint	Name of the Data point
	 * @param	Measurement	Value to associate to the data point
	 * @param	Context		optional context associated with the data point
	 */
	CORE_API virtual void AddTelemetryData(const FString& DataPoint, double Measurement, const FString& Context = TEXT(""));

	/**
	 * Adds several telemetry data point measurements
	 *
	 * @param	ValuePairs	value pair of Name and Measurement of several Data points
	 * @param	Context		optional context associated with the data point
	 */
	CORE_API virtual void AddTelemetryData(const TMap<FString, double>& ValuePairs, const FString& Context = TEXT(""));

	/**
	 * Set telemetry storage name
	 *
	 * @param	StorageName	Name of the data storage
	 */
	CORE_API virtual void SetTelemetryStorage(const FString& StorageName);

	/**
	 * Returns whether this test has any errors associated with it or not
	 *
	 * @return true if this test has at least one error associated with it; false if not
	 */
	CORE_API bool HasAnyErrors() const;

	/**
	* Returns whether this test has encountered all expected log messages defined for it
	* @param VerbosityType Optionally specify to check by log level. Defaults to all.
	* @return true if this test has encountered all expected messages; false if not
	*/
	CORE_API bool HasMetExpectedMessages(ELogVerbosity::Type VerbosityType = ELogVerbosity::All);

	/**
	* Returns whether this test has encountered all expected errors defined for it
	*
	* @return true if this test has encountered all expected errors; false if not
	*/
	CORE_API bool HasMetExpectedErrors();

	/**
	 * Return the last success state for this test
	 */
	CORE_API bool GetLastExecutionSuccessState();

	/**
	 * Populate the provided execution info object with the execution info contained within the test. Not particularly efficient,
	 * but providing direct access to the test's private execution info could result in errors.
	 *
	 * @param	OutInfo	Execution info to be populated with the same data contained within this test's execution info
	 */
	CORE_API void GetExecutionInfo( FAutomationTestExecutionInfo& OutInfo ) const;

	/** 
	 * Helper function that will generate a list of sub-tests via GetTests
	 */
	CORE_API void GenerateTestNames( TArray<FAutomationTestInfo>& TestInfo ) const;

	/**
	 * Helper function that determines if the given log category matches the expected category, inclusively (so an Error counts as a Warning)
	*/
	static CORE_API bool LogCategoryMatchesSeverityInclusive(ELogVerbosity::Type Actual, ELogVerbosity::Type MaximumVerbosity);

	/**
	 * Enables log settings from config
	*/
	static CORE_API void LoadDefaultLogSettings();
	/**
	
	* Adds a regex pattern to an internal list that this test will expect to encounter in logs (of the specified verbosity) during its execution. If an expected pattern
	* is not encountered, it will cause this test to fail.
	*
	* @param ExpectedPatternString - The expected message string. Supports basic regex patterns if IsRegex is set to true (the default).
	* @param ExpectedVerbosity - The expected message verbosity. This is treated as a minimum requirement, so for example the Warning level will intercept Warnings, Errors and Fatal.
	* @param CompareType - How to match this string with an encountered message, should it match exactly or simply just contain the string.
	* @param Occurrences - How many times to expect this message string to be seen. If > 0, the message must be seen the exact number of times
	* specified or the test will fail. If == 0, the message must be seen one or more times (with no upper limit) or the test will fail.
	* If < 0, occurrences of this message will be silently ignored.
	* @param IsRegex - If the pattern is to be used as regex or plain string. Default is true.
	*/
	CORE_API void AddExpectedMessage(FString ExpectedPatternString, ELogVerbosity::Type ExpectedVerbosity, EAutomationExpectedMessageFlags::MatchType CompareType = EAutomationExpectedMessageFlags::Contains, int32 Occurrences = 1, bool IsRegex = true);

	/**

	* Adds a plain string to an internal list that this test will expect to encounter in logs (of the specified verbosity) during its execution. If an expected pattern
	* is not encountered, it will cause this test to fail.
	*
	* @param ExpectedString - The expected message string.
	* @param ExpectedVerbosity - The expected message verbosity. This is treated as a minimum requirement, so for example the Warning level will intercept Warnings, Errors and Fatal.
	* @param CompareType - How to match this string with an encountered message, should it match exactly or simply just contain the string.
	* @param Occurrences - How many times to expect this message string to be seen. If > 0, the message must be seen the exact number of times
	* specified or the test will fail. If == 0, the message must be seen one or more times (with no upper limit) or the test will fail.
	* If < 0, occurrences of this message will be silently ignored.
	*/
	CORE_API void AddExpectedMessagePlain(FString ExpectedString, ELogVerbosity::Type ExpectedVerbosity, EAutomationExpectedMessageFlags::MatchType CompareType = EAutomationExpectedMessageFlags::Contains, int32 Occurrences = 1);

	/**
	* Adds a regex pattern to an internal list that this test will expect to encounter in logs (of all severities) during its execution. If an expected pattern
	* is not encountered, it will cause this test to fail.
	*
	* @param ExpectedPatternString - The expected message string. Supports basic regex patterns.
	* @param CompareType - How to match this string with an encountered message, should it match exactly or simply just contain the string.
	* @param Occurrences - How many times to expect this message string to be seen. If > 0, the message must be seen the exact number of times
	* specified or the test will fail. If == 0, the message must be seen one or more times (with no upper limit) or the test will fail.
	* If < 0, occurrences of this message will be silently ignored.
	* @param IsRegex - If the pattern is to be used as regex or plain string. Default is true.
	*/
	CORE_API void AddExpectedMessage(FString ExpectedPatternString, EAutomationExpectedMessageFlags::MatchType CompareType = EAutomationExpectedMessageFlags::Contains, int32 Occurrences = 1, bool IsRegex = true);

	/**
	* Adds a plain string to an internal list that this test will expect to encounter in logs (of all severities) during its execution. If an expected pattern
	* is not encountered, it will cause this test to fail.
	*
	* @param ExpectedString - The expected message string.
	* @param CompareType - How to match this string with an encountered message, should it match exactly or simply just contain the string.
	* @param Occurrences - How many times to expect this message string to be seen. If > 0, the message must be seen the exact number of times
	* specified or the test will fail. If == 0, the message must be seen one or more times (with no upper limit) or the test will fail.
	* If < 0, occurrences of this message will be silently ignored.
	*/
	CORE_API void AddExpectedMessagePlain(FString ExpectedString, EAutomationExpectedMessageFlags::MatchType CompareType = EAutomationExpectedMessageFlags::Contains, int32 Occurrences = 1);


	/**
	* Populate the provided expected log messages object with the expected messages contained within the test. Not particularly efficient,
	* but providing direct access to the test's private execution messages list could result in errors.
	* @param OutInfo - Array of Expected Messages to be populated with the same data contained within this test's expected messages list
	* @param Verbosity - Optionally filter the returned messages by verbosity. This is inclusive, so Warning will return Warnings, Errors, etc.
	* 
	* @note This will not include any messages that are supposed to be suppressed
	*/
	CORE_API void GetExpectedMessages(TArray<FAutomationExpectedMessage>& OutInfo, ELogVerbosity::Type Verbosity = ELogVerbosity::All) const;

	/**
	* Populate the provided expected log messages object with the expected messages contained within the test. Not particularly efficient,
	* but providing direct access to the test's private execution messages list could result in errors.
	* @param OutInfo - Array of Expected Messages to be populated with the same data contained within this test's expected messages list
	* @param IncludeSuppressed - Include the expected errors that have been marked as being suppressed.
	* @param Verbosity - Optionally filter the returned messages by verbosity. This is inclusive, so Warning will return Warnings, Errors, etc.
	*/
	CORE_API void GetExpectedMessages(TArray<FAutomationExpectedMessage>& OutInfo, bool IncludeSuppressed, ELogVerbosity::Type Verbosity = ELogVerbosity::All) const;

	/**
	* Adds a regex pattern to an internal list that this test will expect to encounter in error or warning logs during its execution. If an expected pattern
	* is not encountered, it will cause this test to fail.
	*
	* @param ExpectedPatternString - The expected message string. Supports basic regex patterns.
	* @param CompareType - How to match this string with an encountered error, should it match exactly or simply just contain the string.
	* @param Occurrences - How many times to expect this error string to be seen. If > 0, the error must be seen the exact number of times
	* specified or the test will fail. If == 0, the error must be seen one or more times (with no upper limit) or the test will fail.
	* If < 0, occurrences of this message will be silently ignored.
	* @param IsRegex - If the pattern is to be used as regex or plain string. Default is true.
	*/
	CORE_API void AddExpectedError(FString ExpectedPatternString, EAutomationExpectedErrorFlags::MatchType CompareType = EAutomationExpectedErrorFlags::Contains, int32 Occurrences = 1, bool IsRegex = true);

	/**
	* Adds a plain string to an internal list that this test will expect to encounter in error or warning logs during its execution. If an expected pattern
	* is not encountered, it will cause this test to fail.
	*
	* @param ExpectedString - The expected message string.
	* @param CompareType - How to match this string with an encountered error, should it match exactly or simply just contain the string.
	* @param Occurrences - How many times to expect this error string to be seen. If > 0, the error must be seen the exact number of times
	* specified or the test will fail. If == 0, the error must be seen one or more times (with no upper limit) or the test will fail.
	* If < 0, occurrences of this message will be silently ignored.
	*/
	CORE_API void AddExpectedErrorPlain(FString ExpectedString, EAutomationExpectedErrorFlags::MatchType CompareType = EAutomationExpectedErrorFlags::Contains, int32 Occurrences = 1);

	/**
	 * Is this a complex tast - if so it will be a stress test.
	 *
	 * @return true if this is a complex task.
	 */
	const bool IsComplexTask() const
	{
		return bComplexTask;
	}

	const bool IsRanOnSeparateThread() const
	{
		return bRunOnSeparateThread;
	}

	/**
	 * If true no logging will be included in test events
	 *
	 * @return true to suppress logs
	 */
	virtual bool SuppressLogs()
	{
		return bSuppressLogs;
	}

	/**
	 * Should the log category be captured and surfaced as part of the test.
	 * If true will then go through the SuppressLogWarnings and SuppressLogErrors checks for if this should be suppressed further or not
	 * Recommend overriding with a virtual function that contains a static TSet to check for the categories you want.
	 * 
	 * @return true to allow a log category through.
	 */
	virtual bool ShouldCaptureLogCategory(const class FName& Category) const { return true; }

	/**
	 * If returns true then logging with a level of Error will not be recorded in test results
	 *
	 * @return false to make errors errors
	 */
	virtual bool SuppressLogErrors() { return bSuppressLogErrors; }

	/**
	 * If returns true then logging with a level of Warning will not be recorded in test results
	 *
	 * @return true to make warnings errors
	 */
	virtual bool SuppressLogWarnings() { return bSuppressLogWarnings; }

	/**
	 * If returns true then logging with a level of Warning will be treated as an error
	 *
	 * @return true to make warnings errors
	 */
	virtual bool ElevateLogWarningsToErrors() { return bElevateLogWarningsToErrors; }

	/**
	 * Return suppressed log categories
	 */
	virtual TArray<FString> GetSuppressedLogCategories() { return SuppressedLogCategories; }


	/**
	 * Enqueues a new latent command.
	 */
	inline void AddCommand(IAutomationLatentCommand* NewCommand)
	{
		TSharedRef<IAutomationLatentCommand> CommandPtr = MakeShareable(NewCommand);
		FAutomationTestFramework::Get().EnqueueLatentCommand(CommandPtr);
	}

	/**
	 * Enqueues a new latent network command.
	 */
	inline void AddCommand(IAutomationNetworkCommand* NewCommand)
	{
		TSharedRef<IAutomationNetworkCommand> CommandPtr = MakeShareable(NewCommand);
		FAutomationTestFramework::Get().EnqueueNetworkCommand(CommandPtr);
	}

	/** Gets the filename where this test was defined. */
	virtual FString GetTestSourceFileName() const { return TEXT(""); }

	/** Gets the line number where this test was defined. */
	virtual int32 GetTestSourceFileLine() const { return 0; }

	/** Gets the filename where this test was defined. */
	virtual FString GetTestSourceFileName(const FString& InTestName) const { return GetTestSourceFileName(); }

	/** Gets the line number where this test was defined. */
	virtual int32 GetTestSourceFileLine(const FString& InTestName) const { return GetTestSourceFileLine(); }

	/** Allows navigation to the asset associated with the test if there is one. */
	virtual FString GetTestAssetPath(const FString& Parameter) const { return TEXT(""); }

	/** Return an exec command to open the test associated with this parameter. */
	virtual FString GetTestOpenCommand(const FString& Parameter) const { return TEXT(""); }

	void PushContext(const FString& Context)
	{
		ExecutionInfo.PushContext(Context);
	}

	void PopContext()
	{
		ExecutionInfo.PopContext();
	}

	/** Checks if the test is able to run in the current environment. */
	virtual bool CanRunInEnvironment(const FString& TestParams, FString* OutReason, bool* OutWarn) const
	{
		// By default the test is able to run in the current environment
		// It is responsibility of a child class to decide if the flow should skip the corresponding test.
		return true;
	}

public:

	CORE_API bool TestEqual(const TCHAR* What, const int32 Actual, const int32 Expected);
	CORE_API bool TestEqual(const TCHAR* What, const int64 Actual, const int64 Expected);
	CORE_API bool TestEqual(const TCHAR* What, const SIZE_T Actual, const SIZE_T Expected);
	CORE_API bool TestEqual(const TCHAR* What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestEqual(const TCHAR* What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestEqual(const TCHAR* What, const FVector Actual, const FVector Expected, float Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestEqual(const TCHAR* What, const FTransform Actual, const FTransform Expected, float Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestEqual(const TCHAR* What, const FRotator Actual, const FRotator Expected, float Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestEqual(const TCHAR* What, const FColor Actual, const FColor Expected);
	CORE_API bool TestEqual(const TCHAR* What, const FLinearColor Actual, const FLinearColor Expected);
	CORE_API bool TestEqual(const TCHAR* What, const TCHAR* Actual, const TCHAR* Expected);
	CORE_API bool TestEqual(const TCHAR* What, FStringView Actual, FStringView Expected);
	bool TestEqual(const TCHAR* What, const FString& Actual, const FString& Expected)
	{
		return TestEqual(What, *Actual, *Expected);
	}
	CORE_API bool TestEqual(const TCHAR* What, FUtf8StringView Actual, FUtf8StringView Expected);
	CORE_API bool TestEqual(const TCHAR* What, const FText Actual, const FText Expected);
	CORE_API bool TestEqual(const TCHAR* What, const FName Actual, const FName Expected);
	CORE_API bool TestNotEqual(const TCHAR* What, const TCHAR* Actual, const TCHAR* Expected);
	CORE_API bool TestNotEqual(const TCHAR* What, FStringView Actual, FStringView Expected);
	bool TestNotEqual(const TCHAR* What, const FString& Actual, const FString& Expected)
	{
		return TestNotEqual(What, *Actual, *Expected);
	}
	CORE_API bool TestNotEqual(const TCHAR* What, FUtf8StringView Actual, FUtf8StringView Expected);
	CORE_API bool TestNotEqual(const TCHAR* What, const FText Actual, const FText Expected);
	CORE_API bool TestNotEqual(const TCHAR* What, const FName Actual, const FName Expected);

	CORE_API bool TestEqualSensitive(const TCHAR* What, const TCHAR* Actual, const TCHAR* Expected);
	CORE_API bool TestEqualSensitive(const TCHAR* What, FStringView Actual, FStringView Expected);
	bool TestEqualSensitive(const TCHAR* What, const FString& Actual, const FString& Expected)
	{
		return TestEqualSensitive(What, *Actual, *Expected);
	}
	CORE_API bool TestEqualSensitive(const TCHAR* What, FUtf8StringView Actual, FUtf8StringView Expected);
	CORE_API bool TestNotEqualSensitive(const TCHAR* What, const TCHAR* Actual, const TCHAR* Expected);
	CORE_API bool TestNotEqualSensitive(const TCHAR* What, FStringView Actual, FStringView Expected);
	bool TestNotEqualSensitive(const TCHAR* What, const FString& Actual, const FString& Expected)
	{
		return TestNotEqualSensitive(What, *Actual, *Expected);
	}
	CORE_API bool TestNotEqualSensitive(const TCHAR* What, FUtf8StringView Actual, FUtf8StringView Expected);

	bool TestEqual(const FString& What, const int32 Actual, const int32 Expected)
	{
		return TestEqual(*What, Actual, Expected);
	}

	bool TestEqual(const FString& What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestEqual(const FString& What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestEqual(const FString& What, const FVector Actual, const FVector Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestEqual(const FString& What, const FTransform Actual, const FTransform Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestEqual(const FString& What, const FRotator Actual, const FRotator Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestEqual(const FString& What, const FColor Actual, const FColor Expected)
	{
		return TestEqual(*What, Actual, Expected);
	}

	bool TestEqual(const FString& What, const TCHAR* Actual, const TCHAR* Expected)
	{
		return TestEqual(*What, Actual, Expected);
	}

	bool TestEqual(const FString& What, FUtf8StringView Actual, FUtf8StringView Expected)
	{
		return TestEqual(*What, Actual, Expected);
	}

	bool TestEqual(const FString& What, FStringView Actual, FStringView Expected)
	{
		return TestEqual(*What, Actual, Expected);
	}

	bool TestEqual(const FString& What, FStringView Actual, FUtf8StringView Expected)
	{
		// This overload is here because there are some tests that have raw string literals as the expected value.
		FString Tmp(Expected);
		return TestEqual(*What, Actual, Tmp);
	}

	bool TestEqual(const FString& What, FUtf8StringView Actual, FStringView Expected)
	{
		// This overload is here because there are some tests that have raw string literals as the Actual value.
		FString Tmp(Actual);
		return TestEqual(*What, Tmp, Expected);
	}

	bool TestEqualSensitive(const FString& What, const TCHAR* Actual, const TCHAR* Expected)
	{
		return TestEqualSensitive(*What, Actual, Expected);
	}

	bool TestEqualSensitive(const FString& What, FUtf8StringView Actual, FUtf8StringView Expected)
	{
		return TestEqualSensitive(*What, Actual, Expected);
	}

	bool TestEqualSensitive(const FString& What, FStringView Actual, FStringView Expected)
	{
		return TestEqualSensitive(*What, Actual, Expected);
	}

	bool TestEqualSensitive(const FString& What, FStringView Actual, FUtf8StringView Expected)
	{
		// This overload is here because there are some tests that have raw string literals as the expected value.
		FString Tmp(Expected);
		return TestEqualSensitive(*What, Actual, Tmp);
	}

	bool TestEqualSensitive(const FString& What, FUtf8StringView Actual, FStringView Expected)
	{
		// This overload is here because there are some tests that have raw string literals as the Actual value.
		FString Tmp(Actual);
		return TestEqualSensitive(*What, Tmp, Expected);
	}

	bool TestNotEqualSensitive(const FString& What, const TCHAR* Actual, const TCHAR* Expected)
	{
		return TestNotEqualSensitive(*What, Actual, Expected);
	}

	bool TestNotEqualSensitive(const FString& What, FUtf8StringView Actual, FUtf8StringView Expected)
	{
		return TestNotEqualSensitive(*What, Actual, Expected);
	}

	bool TestNotEqualSensitive(const FString& What, FStringView Actual, FStringView Expected)
	{
		return TestNotEqualSensitive(*What, Actual, Expected);
	}

	bool TestNotEqualSensitive(const FString& What, FStringView Actual, FUtf8StringView Expected)
	{
		// This overload is here because there are some tests that have raw string literals as the expected value.
		FString Tmp(Expected);
		return TestNotEqualSensitive(*What, Actual, Tmp);
	}

	bool TestNotEqualSensitive(const FString& What, FUtf8StringView Actual, FStringView Expected)
	{
		// This overload is here because there are some tests that have raw string literals as the Actual value.
		FString Tmp(Actual);
		return TestNotEqualSensitive(*What, Tmp, Expected);
	}

	struct FHasToString
	{
		template<typename ValueType>
		auto Requires(ValueType& InElem) -> decltype(InElem.ToString());
	};

	template<typename ValueType> 
	inline void ReportError(const TCHAR* What, const ValueType& Actual, const ValueType& Expected)
	{
		if constexpr (TModels_V<FHasToString, ValueType>)
		{
			ReportError(What, *Actual.ToString(), *Expected.ToString());
		}
		else
		{
			AddError(FString::Printf(TEXT("%s: The two values are not equal."), What));
		}
	}

	template<> 
	inline void ReportError(const TCHAR* What, const FString& Actual, const FString& Expected)
	{
		AddError(FString::Printf(TEXT("%s: The two values are not equal (actual=%s, expected=%s)."), What, *Actual, *Expected));
	}

	/**
	 * Logs an error if the two values are not equal.
	 *
	 * @param What - Description text for the test.
	 * @param A - The first value.
	 * @param B - The second value.
	 *
	 * @see TestNotEqual
	 */
	template<typename ValueType> 
	inline bool TestEqual(const TCHAR* What, const ValueType& Actual, const ValueType& Expected)
	{
		if (Actual != Expected)
		{
			ReportError(What, Actual, Expected);
			return false;
		}
		return true;
	}

	template<typename ValueType>
	bool TestEqual(const FString& What, const ValueType& Actual, const ValueType& Expected)
	{
		return TestEqual(*What, Actual, Expected);
	}

	CORE_API bool TestNearlyEqual(const TCHAR* What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestNearlyEqual(const TCHAR* What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestNearlyEqual(const TCHAR* What, const FVector Actual, const FVector Expected, float Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestNearlyEqual(const TCHAR* What, const FTransform Actual, const FTransform Expected, float Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestNearlyEqual(const TCHAR* What, const FRotator Actual, const FRotator Expected, float Tolerance = UE_KINDA_SMALL_NUMBER);

	bool TestNearlyEqual(const FString& What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestNearlyEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestNearlyEqual(const FString& What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestNearlyEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestNearlyEqual(const FString& What, const FVector Actual, const FVector Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestNearlyEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestNearlyEqual(const FString& What, const FTransform Actual, const FTransform Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestNearlyEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestNearlyEqual(const FString& What, const FRotator Actual, const FRotator Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestNearlyEqual(*What, Actual, Expected, Tolerance);
	}

	CORE_API bool TestLessThan(const TCHAR* What, const int32 Actual, const int32 Expected);
	CORE_API bool TestLessThan(const TCHAR* What, const int64 Actual, const int64 Expected);
	CORE_API bool TestGreaterThan(const TCHAR* What, const int32 Actual, const int32 Expected);
	CORE_API bool TestGreaterThan(const TCHAR* What, const int64 Actual, const int64 Expected);
	CORE_API bool TestLessEqual(const TCHAR* What, const int32 Actual, const int32 Expected);
	CORE_API bool TestLessEqual(const TCHAR* What, const int64 Actual, const int64 Expected);
	CORE_API bool TestGreaterEqual(const TCHAR* What, const int32 Actual, const int32 Expected);
	CORE_API bool TestGreaterEqual(const TCHAR* What, const int64 Actual, const int64 Expected);
	CORE_API bool TestLessThan(const TCHAR* What, const SIZE_T Actual, const SIZE_T Expected);
	CORE_API bool TestGreaterThan(const TCHAR* What, const SIZE_T Actual, const SIZE_T Expected);
	CORE_API bool TestLessEqual(const TCHAR* What, const SIZE_T Actual, const SIZE_T Expected);
	CORE_API bool TestGreaterEqual(const TCHAR* What, const SIZE_T Actual, const SIZE_T Expected);
	CORE_API bool TestLessThan(const TCHAR* What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestLessThan(const TCHAR* What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestGreaterThan(const TCHAR* What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestGreaterThan(const TCHAR* What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestLessEqual(const TCHAR* What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestLessEqual(const TCHAR* What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestGreaterEqual(const TCHAR* What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestGreaterEqual(const TCHAR* What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER);

	bool TestLessThan(const FString& What, const int32 Actual, const int32 Expected)
	{
		return TestLessThan(*What, Actual, Expected);
	}

	bool TestLessThan(const FString& What, const int64 Actual, const int64 Expected)
	{
		return TestLessThan(*What, Actual, Expected);
	}

	bool TestGreaterThan(const FString& What, const int32 Actual, const int32 Expected)
	{
		return TestGreaterThan(*What, Actual, Expected);
	}

	bool TestGreaterThan(const FString& What, const int64 Actual, const int64 Expected)
	{
		return TestGreaterThan(*What, Actual, Expected);
	}

	bool TestLessEqual(const FString& What, const int32 Actual, const int32 Expected)
	{
		return TestLessEqual(*What, Actual, Expected);
	}

	bool TestLessEqual(const FString& What, const int64 Actual, const int64 Expected)
	{
		return TestLessEqual(*What, Actual, Expected);
	}

	bool TestGreaterEqual(const FString& What, const int32 Actual, const int32 Expected)
	{
		return TestGreaterEqual(*What, Actual, Expected);
	}

	bool TestGreaterEqual(const FString& What, const int64 Actual, const int64 Expected)
	{
		return TestGreaterEqual(*What, Actual, Expected);
	}

	bool TestLessThan(const FString& What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestLessThan(*What, Actual, Expected, Tolerance);
	}

	bool TestLessThan(const FString& What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestLessThan(*What, Actual, Expected, Tolerance);
	}

	bool TestGreaterThan(const FString& What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestGreaterThan(*What, Actual, Expected, Tolerance);
	}

	bool TestGreaterThan(const FString& What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestGreaterThan(*What, Actual, Expected, Tolerance);
	}

	bool TestLessEqual(const FString& What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestLessEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestLessEqual(const FString& What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestLessEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestGreaterEqual(const FString& What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestGreaterEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestGreaterEqual(const FString& What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestGreaterEqual(*What, Actual, Expected, Tolerance);
	}

	CORE_API bool TestBetweenExclusive(const TCHAR* What, const int32 Actual, const int32 ExpectedLow, const int32 ExpectedHigh);
	CORE_API bool TestBetweenExclusive(const TCHAR* What, const int64 Actual, const int64 ExpectedLow, const int64 ExpectedHigh);
	CORE_API bool TestBetweenInclusive(const TCHAR* What, const int32 Actual, const int32 ExpectedLow, const int32 ExpectedHigh);
	CORE_API bool TestBetweenInclusive(const TCHAR* What, const int64 Actual, const int64 ExpectedLow, const int64 ExpectedHigh);

	bool TestBetweenExclusive(const FString& What, const int32 Actual, const int32 ExpectedLow, const int32 ExpectedHigh)
	{
		return TestBetweenExclusive(*What, Actual, ExpectedLow, ExpectedHigh);
	}

	bool TestBetweenExclusive(const FString& What, const int64 Actual, const int64 ExpectedLow, const int64 ExpectedHigh)
	{
		return TestBetweenExclusive(*What, Actual, ExpectedLow, ExpectedHigh);
	}

	bool TestBetweenInclusive(const FString& What, const int32 Actual, const int32 ExpectedLow, const int32 ExpectedHigh)
	{
		return TestBetweenInclusive(*What, Actual, ExpectedLow, ExpectedHigh);
	}

	bool TestBetweenInclusive(const FString& What, const int64 Actual, const int64 ExpectedLow, const int64 ExpectedHigh)
	{
		return TestBetweenInclusive(*What, Actual, ExpectedLow, ExpectedHigh);
	}

	/**
	 * Logs an error if the specified Boolean value is not false.
	 *
	 * @param What - Description text for the test.
	 * @param Value - The value to test.
	 *
	 * @see TestTrue
	 */
	CORE_API bool TestFalse(const TCHAR* What, bool Value);

	bool TestFalse(const FString& What, bool Value)
	{
		return TestFalse(*What, Value);
	}

	/** Macro version of above, uses the passed in expression as the description as well */
	#define TestFalseExpr(Expression) TestFalse(TEXT(#Expression), Expression)

	/**
	 * Logs an error if the given object tests true when calling its IsValid member.
	 *
	 * @param Description - Description text for the test.
	 * @param Value - The value to test.
	 *
	 * @see TestValid
	 */
	template<typename ValueType>
	inline bool TestInvalid(const TCHAR* Description, const ValueType& Value)
	{
		if (Value.IsValid())
		{
			AddError(FString::Printf(TEXT("%s: The value is valid (.IsValid() returned true)."), Description));
			return false;
		}
		return true;
	}

	template<typename ValueType>
	bool TestInvalid(const FString& Description, const ValueType& Value)
	{
		return TestInvalid(*Description, Value);
	}

	/**
	 * Logs an error if the two values are equal.
	 *
	 * @param Description - Description text for the test.
	 * @param A - The first value.
	 * @param B - The second value.
	 *
	 * @see TestEqual
	 */
	template<typename ValueType>
	inline bool TestNotEqual(const TCHAR* Description, const ValueType& Actual, const ValueType& Expected)
	{
		if (Actual == Expected)
		{
			AddError(FString::Printf(TEXT("%s: The two values are equal."), Description));
			return false;
		}
		return true;
	}

	template<typename ValueType> bool TestNotEqual(const FString& Description, const ValueType& Actual, const ValueType& Expected)
	{
		return TestNotEqual(*Description, Actual, Expected);
	}

	CORE_API bool TestNotEqual(const TCHAR* What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER);
	CORE_API bool TestNotEqual(const TCHAR* What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER);

	bool TestNotEqual(const FString& What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestNotEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestNotEqual(const FString& What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestNotEqual(*What, Actual, Expected, Tolerance);
	}

	/**
	 * Logs an error if the specified pointer is NULL.
	 *
	 * @param What - Description text for the test.
	 * @param Pointer - The pointer to test.
	 *
	 * @see TestNull
	 */
	template<typename ValueType>
	inline bool TestNotNull(const TCHAR* What, const ValueType* Pointer)
	{
		if (Pointer == nullptr)
		{
			AddError(FString::Printf(TEXT("Expected '%s' to be not null."), What));
			return false;
		}
		return true;
	}

	template<typename ValueType> bool TestNotNull(const FString& What, const ValueType* Pointer)
	{
		return TestNotNull(*What, Pointer);
	}

	/**
	 * Logs an error if the two values reference the same object in memory.
	 *
	 * @param Description - Description text for the test.
	 * @param A - The first reference.
	 * @param B - The second reference.
	 *
	 * @see TestSame, TestNotSamePtr
	 */
	template<typename ValueType>
	inline bool TestNotSame(const TCHAR* Description, const ValueType& Actual, const ValueType& Expected)
	{
		if (&Actual == &Expected)
		{
			AddError(FString::Printf(TEXT("%s: The two values are the same."), Description));
			return false;
		}
		return true;
	}

	template<typename ValueType> bool TestNotSame(const FString& Description, const ValueType& Actual, const ValueType& Expected)
	{
		return TestNotSame(*Description, Actual, Expected);
	}

	/**
	 * Logs an error if the two pointers point to the same object in memory.
	 *
	 * @param Description - Description text for the test.
	 * @param A - The first pointer.
	 * @param B - The second pointer.
	 *
	 * @see TestNotSamePtr, TestSame
	 */
	template<typename ValueType>
	inline bool TestNotSamePtr(const TCHAR* Description, const ValueType* Actual, const ValueType* Expected)
	{
		if (nullptr == Actual)
		{
			AddWarning(FString::Printf(TEXT("%s: Actual value is nullptr."), Description));
		}
		if (nullptr == Expected)
		{
			AddWarning(FString::Printf(TEXT("%s: Expected value is nullptr, which may be unintended. If intentional consider instead using TestNotNull()."), Description));
		}
		if (Actual == Expected)
		{
			AddError(FString::Printf(TEXT("%s: The two pointers are the same."), Description));
			return false;
		}
		return true;
	}

	template<typename ValueType> bool TestNotSamePtr(const FString& Description, const ValueType* Actual, const ValueType* Expected)
	{
		return TestNotSamePtr(*Description, Actual, Expected);
	}

	/**
	 * Logs an error if the specified pointer is not NULL.
	 *
	 * @param Description - Description text for the test.
	 * @param Pointer - The pointer to test.
	 *
	 * @see TestNotNull
	 */
	CORE_API bool TestNull(const TCHAR* What, const void* Pointer);

	bool TestNull(const FString& What, const void* Pointer)
	{
		return TestNull(*What, Pointer);
	}

	/**
	 * Logs an error if the two values do not reference the same object in memory.
	 *
	 * @param Description - Description text for the test.
	 * @param Actual - The actual reference.
	 * @param Expected - The expected reference.
	 *
	 * @see TestNotSame, TestSamePtr
	 */
	template<typename ValueType>
	inline bool TestSame(const TCHAR* Description, const ValueType& Actual, const ValueType& Expected)
	{
		if (&Actual != &Expected)
		{
			AddError(FString::Printf(TEXT("%s: The two values are not the same."), Description));
			return false;
		}
		return true;
	}

	template<typename ValueType> bool TestSame(const FString& Description, const ValueType& Actual, const ValueType& Expected)
	{
		return TestSame(*Description, Actual, Expected);
	}

	/**
	 * Logs an error if the two pointers do not point to the same object in memory.
	 *
	 * @param Description - Description text for the test.
	 * @param Actual - The actual pointer.
	 * @param Expected - The expected pointer.
	 *
	 * @see TestNotSamePtr, TestNotSame
	 */
	template<typename ValueType>
	inline bool TestSamePtr(const TCHAR* Description, const ValueType* Actual, const ValueType* Expected)
	{
		if (nullptr == Actual)
		{
			AddWarning(FString::Printf(TEXT("%s: Actual value is nullptr."), Description));
		}
		if (nullptr == Expected)
		{
			AddWarning(FString::Printf(TEXT("%s: Expected value is nullptr, which may be unintended. If intentional consider instead using TestNull()."), Description));
		}
		if (Actual != Expected)
		{
			AddError(FString::Printf(TEXT("%s: The two pointers are not the same."), Description));
			return false;
		}
		return true;
	}

	template<typename ValueType> bool TestSamePtr(const FString& Description, const ValueType* Actual, const ValueType* Expected)
	{
		return TestSamePtr(*Description, Actual, Expected);
	}

	/**
	 * Logs an error if the specified Boolean value is not true.
	 *
	 * @param Description - Description text for the test.
	 * @param Value - The value to test.
	 *
	 * @see TestFalse
	 */
	CORE_API bool TestTrue(const TCHAR* What, bool Value);

	bool TestTrue(const FString& What, bool Value)
	{
		return TestTrue(*What, Value);
	}

	/** Macro version of above, uses the passed in expression as the description as well */
	#define TestTrueExpr(...) TestTrue(TEXT(#__VA_ARGS__), __VA_ARGS__)

	/**
	 * Logs an error if the given object returns false when calling its IsValid member.
	 *
	 * @param Description - Description text for the test.
	 * @param Value - The value to test.
	 *
	 * @see TestInvalid
	 */
	template<typename ValueType>
	inline bool TestValid(const TCHAR* Description, const ValueType& Value)
	{
		if (!Value.IsValid())
		{
			AddError(FString::Printf(TEXT("%s: The value is not valid (.IsValid() returned false)."), Description));
			return false;
		}
		return true;
	}

	template<typename ValueType>
	bool TestValid(const FString& Description, const ValueType& Value)
	{
		return TestValid(*Description, Value);
	}

protected:
	/**
	 * Asks the test to enumerate variants that will all go through the "RunTest" function with different parameters (for load all maps, this should enumerate all maps to load)\
	 *
	 * @param OutBeautifiedNames - Name of the test that can be displayed by the UI (for load all maps, it would be the map name without any directory prefix)
	 * @param OutTestCommands - The parameters to be specified to each call to RunTests (for load all maps, it would be the map name to load)
	 */
	virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const = 0;

	/**
	 * Virtual call to execute the automation test.  
	 *
	 * @param Parameters - Parameter list for the test (but it will be empty for simple tests)
	 * @return TRUE if the test was run successfully; FALSE otherwise
	 */
	virtual bool RunTest(const FString& Parameters)=0;

	/** Sets the parameter context of the test. */
	virtual void SetTestContext(FString Context) { TestParameterContext = Context; }

	/** Extracts a combined EAutomationTestFlags value from a string representation using tag notation "[Filter_1]...[Filter_n][Tag_1]...[Tag_m]" */
	CORE_API EAutomationTestFlags ExtractAutomationTestFlags(FString InTagNotation);

protected:

	//Flag to indicate if this is a complex task
	bool bComplexTask;

	//Flag to indicate if this test should be ran on it's own thread
	bool bRunOnSeparateThread;

	/** Flag to suppress logs */
	bool bSuppressLogs = false;

	/** Name of the test */
	FString TestName;

	/** Context of the test */
	FString TestParameterContext;

	/** Info related to the last execution of this test */
	FAutomationTestExecutionInfo ExecutionInfo;

	//allow framework to call protected function
	friend class FAutomationTestFramework;

private:
	/**
	* Returns whether this test has defined any expected log messages matching the given message.
	* If a match is found, the expected message definition increments it actual occurrence count.
	*
	* @return true if this message matches any of the expected messages
	*/
	CORE_API bool IsExpectedMessage(const FString& Message, const ELogVerbosity::Type& Verbosity = ELogVerbosity::All);

	/**
	 * Sets whether the test has succeeded or not
	 *
	 * @param	bSuccessful	true to mark the test successful, false to mark the test as failed
	 */
	CORE_API void InternalSetSuccessState(bool bSuccessful);

	/**
	 * Returns a string description of the value stored in the passed string view in a form suitable for error messages.
	 * Proper string values will be output with quotation marks e.g. "value".
	 * Null will be represented as nullptr.
	 */
	FString GetStringValueToDisplay(FStringView Value) const;
	FString GetStringValueToDisplay(FUtf8StringView Value) const;

	/* Log messages to be expected while processing this test.*/
	TSet<FAutomationExpectedMessage> ExpectedMessages;

	/** Critical section lock */
	FTransactionallySafeRWLock ActionCS;
};

class FBDDAutomationTestBase : public FAutomationTestBase
{ 
public:
	FBDDAutomationTestBase(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask) 
		, bIsDiscoveryMode(false)
		, bBaseRunTestRan(false)
	{}

	virtual bool RunTest(const FString& Parameters) override
	{
		bBaseRunTestRan = true;
		TestIdToExecute = Parameters;

		return true;
	}

	virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const override
	{
		const_cast<FBDDAutomationTestBase*>(this)->BeautifiedNames.Empty();
		const_cast<FBDDAutomationTestBase*>(this)->TestCommands.Empty();

		bIsDiscoveryMode = true;
		const_cast<FBDDAutomationTestBase*>(this)->RunTest(FString());
		bIsDiscoveryMode = false;

		OutBeautifiedNames.Append(BeautifiedNames);
		OutTestCommands.Append(TestCommands);
		bBaseRunTestRan = false;
	}

	bool IsDiscoveryMode() const
	{
		return bIsDiscoveryMode;
	}

	void xDescribe(const FString& InDescription, TFunction<void()> DoWork)
	{
		check(bBaseRunTestRan);
		//disabled this suite
	}

	void Describe(const FString& InDescription, TFunction<void()> DoWork)
	{
		check(bBaseRunTestRan);

		PushDescription(InDescription);

		int32 OriginalBeforeEachCount = BeforeEachStack.Num();
		int32 OriginalAfterEachCount = AfterEachStack.Num();

		DoWork();

		check(OriginalBeforeEachCount <= BeforeEachStack.Num());
		if (OriginalBeforeEachCount != BeforeEachStack.Num())
		{
			BeforeEachStack.Pop();
		}

		check(OriginalAfterEachCount <= AfterEachStack.Num());
		if (OriginalAfterEachCount != AfterEachStack.Num())
		{
			AfterEachStack.Pop();
		}

		PopDescription(InDescription);
	}
	
	void xIt(const FString& InDescription, TFunction<void()> DoWork)
	{
		check(bBaseRunTestRan);
		//disabled this spec
	}

	void It(const FString& InDescription, TFunction<void()> DoWork)
	{
		check(bBaseRunTestRan);

		PushDescription(InDescription);

		if (bIsDiscoveryMode)
		{
			BeautifiedNames.Add(GetDescription());

			bIsDiscoveryMode = false;
			TestCommands.Add(GetDescription());
			bIsDiscoveryMode = true;
		}
		else if (TestIdToExecute.IsEmpty() || GetDescription() == TestIdToExecute)
		{
			for (int32 Index = 0; Index < BeforeEachStack.Num(); Index++)
			{
				BeforeEachStack[Index]();
			}

			DoWork();

			for (int32 Index = AfterEachStack.Num() - 1; Index >= 0; Index--)
			{
				AfterEachStack[Index]();
			}
		}

		PopDescription(InDescription);
	}

	void BeforeEach(TFunction<void()> DoWork)
	{
		BeforeEachStack.Push(DoWork);
	}

	void AfterEach(TFunction<void()> DoWork)
	{
		AfterEachStack.Push(DoWork);
	}

private:

	void PushDescription(const FString& InDescription)
	{
		Description.Add(InDescription);
	}

	void PopDescription(const FString& InDescription)
	{
		Description.RemoveAt(Description.Num() - 1);
	}

	FString GetDescription() const
	{
		FString CompleteDescription;
		for (int32 Index = 0; Index < Description.Num(); ++Index)
		{
			if (Description[Index].IsEmpty())
			{
				continue;
			}

			if (CompleteDescription.IsEmpty())
			{
				CompleteDescription = Description[Index];
			}
			else if (FChar::IsWhitespace(CompleteDescription[CompleteDescription.Len() - 1]) || FChar::IsWhitespace(Description[Index][0]))
			{
				if (bIsDiscoveryMode)
				{
					CompleteDescription = CompleteDescription + TEXT(".") + Description[Index];
				}
				else
				{
					CompleteDescription = CompleteDescription + Description[Index];
				}
			}
			else
			{
				if (bIsDiscoveryMode)
				{
					CompleteDescription = FString::Printf(TEXT("%s.%s"), *CompleteDescription, *Description[Index]);
				}
				else
				{
					CompleteDescription = FString::Printf(TEXT("%s %s"), *CompleteDescription, *Description[Index]);
				}
			}
		}

		return CompleteDescription;
	}

private:

	FString TestIdToExecute;
	TArray<FString> Description;
	TArray<TFunction<void()>> BeforeEachStack;
	TArray<TFunction<void()>> AfterEachStack;

	TArray<FString> BeautifiedNames;
	TArray<FString> TestCommands;
	mutable bool bIsDiscoveryMode;
	mutable bool bBaseRunTestRan;
};

DECLARE_DELEGATE(FDoneDelegate);

class FAutomationSpecBase 
	: public FAutomationTestBase
	, public TSharedFromThis<FAutomationSpecBase>
{
private:

	class FSingleExecuteLatentCommand : public IAutomationLatentCommand
	{
	public:
		FSingleExecuteLatentCommand(const FAutomationSpecBase* const InSpec, TFunction<void()> InPredicate, bool bInSkipIfErrored = false)
			: Spec(InSpec)
			, Predicate(MoveTemp(InPredicate))
			, bSkipIfErrored(bInSkipIfErrored)
		{ }

		virtual ~FSingleExecuteLatentCommand() = default;

		virtual bool Update() override
		{
			if (bSkipIfErrored && Spec->HasAnyErrors())
			{
				return true;
			}

			Predicate();
			return true;
		}

	private:

		const FAutomationSpecBase* const Spec;
		const TFunction<void()> Predicate;
		const bool bSkipIfErrored;
	};

	class FUntilDoneLatentCommand : public IAutomationLatentCommand
	{
	public:

		FUntilDoneLatentCommand(FAutomationSpecBase* const InSpec, TFunction<void(const FDoneDelegate&)> InPredicate, const FTimespan& InTimeout, bool bInSkipIfErrored = false)
			: Spec(InSpec)
			, Predicate(MoveTemp(InPredicate))
			, Timeout(InTimeout)
			, bSkipIfErrored(bInSkipIfErrored)
			, bIsRunning(false)
			, bDone(false)
		{ }

		virtual ~FUntilDoneLatentCommand() = default;

		virtual bool Update() override
		{
			if (!bIsRunning)
			{
				if (bSkipIfErrored && Spec->HasAnyErrors())
				{
					return true;
				}

				bDone = false;
				bIsRunning = true;
				StartedRunning = FDateTime::UtcNow();
				Predicate(FDoneDelegate::CreateSP(this, &FUntilDoneLatentCommand::Done));
			}

			if (bDone)
			{
				Reset();
				return true;
			}
			else if ((FDateTime::UtcNow() >= StartedRunning + Timeout) && !FPlatformMisc::IsDebuggerPresent())
			{
				Reset();
				Spec->AddError(TEXT("Latent command timed out."), 0);
				return true;
			}

			return false;
		}

	private:

		void Done()
		{
			if (bIsRunning)
			{
				bDone = true;
			}
		}

		void Reset()
		{
			// Reset the done for the next potential run of this command
			bDone = false;
			bIsRunning = false;
		}

	private:

		FAutomationSpecBase* const Spec;
		const TFunction<void(const FDoneDelegate&)> Predicate;
		const FTimespan Timeout;
		const bool bSkipIfErrored;

		bool bIsRunning;
		FDateTime StartedRunning;
		FThreadSafeBool bDone;
	};

	class FAsyncUntilDoneLatentCommand : public IAutomationLatentCommand
	{
	public:

		FAsyncUntilDoneLatentCommand(FAutomationSpecBase* const InSpec, EAsyncExecution InExecution, TFunction<void(const FDoneDelegate&)> InPredicate, const FTimespan& InTimeout, bool bInSkipIfErrored = false)
			: Spec(InSpec)
			, Execution(InExecution)
			, Predicate(MoveTemp(InPredicate))
			, Timeout(InTimeout)
			, bSkipIfErrored(bInSkipIfErrored)
			, bDone(false)
		{ }

		virtual ~FAsyncUntilDoneLatentCommand() = default;

		virtual bool Update() override
		{
			if (!Future.IsValid())
			{
				if (bSkipIfErrored && Spec->HasAnyErrors())
				{
					return true;
				}

				UE::TScopeLock CriticalSection(ActionCS);
				bDone = false;
				StartedRunning = FDateTime::UtcNow();
				Future = Async(Execution, [this]()
				{
					Predicate(FDoneDelegate::CreateRaw(this, &FAsyncUntilDoneLatentCommand::Done));
				});
			}

			if (bDone)
			{
				Reset();
				return true;
			}
			else if ((FDateTime::UtcNow() >= StartedRunning + Timeout) && !FPlatformMisc::IsDebuggerPresent())
			{
				Reset();
				Spec->AddError(TEXT("Latent command timed out."), 0);
				return true;
			}

			return false;
		}

	private:

		void Done()
		{
			UE::TScopeLock CriticalSection(ActionCS);
			if (Future.IsValid())
			{
				bDone = true;
			}
		}

		void Reset()
		{
			UE::TScopeLock CriticalSection(ActionCS);
			// Reset the done for the next potential run of this command
			bDone = false;
			Future.Reset();
		}

	private:

		FAutomationSpecBase* const Spec;
		const EAsyncExecution Execution;
		const TFunction<void(const FDoneDelegate&)> Predicate;
		const FTimespan Timeout;
		const bool bSkipIfErrored;

		FThreadSafeBool bDone;
		FDateTime StartedRunning;
		TFuture<void> Future;
		FTransactionallySafeCriticalSection ActionCS;
	};

	class FAsyncLatentCommand : public IAutomationLatentCommand
	{
	public:

		FAsyncLatentCommand(FAutomationSpecBase* const InSpec, EAsyncExecution InExecution, TFunction<void()> InPredicate, const FTimespan& InTimeout, bool bInSkipIfErrored = false)
			: Spec(InSpec)
			, Execution(InExecution)
			, Predicate(MoveTemp(InPredicate))
			, Timeout(InTimeout)
			, bSkipIfErrored(bInSkipIfErrored)
			, bDone(false)
		{ }

		virtual ~FAsyncLatentCommand() = default;

		virtual bool Update() override
		{
			if (!Future.IsValid())
			{
				if (bSkipIfErrored && Spec->HasAnyErrors())
				{
					return true;
				}

				UE::TScopeLock CriticalSection(ActionCS);
				bDone = false;
				StartedRunning = FDateTime::UtcNow();
				Future = Async(Execution, [this]()
				{
					Predicate();
					Done();
				});
			}

			if (bDone)
			{
				Reset();
				return true;
			}
			else if ((FDateTime::UtcNow() >= StartedRunning + Timeout) && !FPlatformMisc::IsDebuggerPresent())
			{
				Reset();
				Spec->AddError(TEXT("Latent command timed out."), 0);
				return true;
			}

			return false;
		}

	private:

		void Done()
		{
			UE::TScopeLock CriticalSection(ActionCS);
			if (Future.IsValid())
			{
				bDone = true;
			}
		}

		void Reset()
		{
			UE::TScopeLock CriticalSection(ActionCS);
			// Reset the done for the next potential run of this command
			bDone = false;
			Future.Reset();
		}

	private:

		FAutomationSpecBase* const Spec;
		const EAsyncExecution Execution;
		const TFunction<void()> Predicate;
		const FTimespan Timeout;
		const bool bSkipIfErrored;

		FThreadSafeBool bDone;
		FDateTime StartedRunning;
		TFuture<void> Future;
		FTransactionallySafeCriticalSection ActionCS;
	};

	struct FSpecIt
	{
	public:

		FString Description;
		FString Id;
		FString Filename;
		int32 LineNumber;
		TSharedRef<IAutomationLatentCommand> Command;

		FSpecIt(FString InDescription, FString InId, FString InFilename, int32 InLineNumber, TSharedRef<IAutomationLatentCommand> InCommand)
			: Description(MoveTemp(InDescription))
			, Id(MoveTemp(InId))
			, Filename(InFilename)
			, LineNumber(MoveTemp(InLineNumber))
			, Command(MoveTemp(InCommand))
		{ }
	};

	struct FSpecDefinitionScope
	{
	public:

		FString Description;

		TArray<TSharedRef<IAutomationLatentCommand>> BeforeEach;
		TArray<TSharedRef<FSpecIt>> It;
		TArray<TSharedRef<IAutomationLatentCommand>> AfterEach;

		TArray<TSharedRef<FSpecDefinitionScope>> Children;
	};

	struct FSpec
	{
	public:

		FString Id;
		FString Description;
		FString Filename;
		int32 LineNumber;
		TArray<TSharedRef<IAutomationLatentCommand>> Commands;
	};

public:

	FAutomationSpecBase(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
		, DefaultTimeout(FTimespan::FromSeconds(30))
		, bEnableSkipIfError(true)
		, RootDefinitionScope(MakeShareable(new FSpecDefinitionScope()))
		, bHasBeenDefined(false)
	{
		DefinitionScopeStack.Push(RootDefinitionScope.ToSharedRef());
	}

	virtual bool RunTest(const FString& InParameters) override
	{
		EnsureDefinitions();

		if (!InParameters.IsEmpty())
		{
			const TSharedRef<FSpec>* SpecToRun = IdToSpecMap.Find(InParameters);
			if (SpecToRun != nullptr)
			{
				for (int32 Index = 0; Index < (*SpecToRun)->Commands.Num(); ++Index)
				{
					FAutomationTestFramework::GetInstance().EnqueueLatentCommand((*SpecToRun)->Commands[Index]);
				}
			}
		}
		else
		{
			TArray<TSharedRef<FSpec>> Specs;
			IdToSpecMap.GenerateValueArray(Specs);

			for (int32 SpecIndex = 0; SpecIndex < Specs.Num(); SpecIndex++)
			{
				for (int32 CommandIndex = 0; CommandIndex < Specs[SpecIndex]->Commands.Num(); ++CommandIndex)
				{
					FAutomationTestFramework::GetInstance().EnqueueLatentCommand(Specs[SpecIndex]->Commands[CommandIndex]);
				}
			}
		}

		return true;
	}

	virtual bool IsStressTest() const 
	{
		return false; 
	}

	virtual uint32 GetRequiredDeviceNum() const override 
	{ 
		return 1; 
	}

	virtual FString GetTestSourceFileName(const FString& InTestName) const override
	{
		FString TestId = InTestName;
		if (TestId.StartsWith(TestName + TEXT(" ")))
		{
			TestId = InTestName.RightChop(TestName.Len() + 1);
		}

		const TSharedRef<FSpec>* Spec = IdToSpecMap.Find(TestId);
		if (Spec != nullptr)
		{
			return (*Spec)->Filename;
		}

		return FAutomationTestBase::GetTestSourceFileName();
	}

	virtual int32 GetTestSourceFileLine(const FString& InTestName) const override
	{ 
		FString TestId = InTestName;
		if (TestId.StartsWith(TestName + TEXT(" ")))
		{
			TestId = InTestName.RightChop(TestName.Len() + 1);
		}

		const TSharedRef<FSpec>* Spec = IdToSpecMap.Find(TestId);
		if (Spec != nullptr)
		{
			return (*Spec)->LineNumber;
		}

		return FAutomationTestBase::GetTestSourceFileLine();
	}

	virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const override
	{
		EnsureDefinitions();

		TArray<TSharedRef<FSpec>> Specs;
		IdToSpecMap.GenerateValueArray(Specs);

		for (int32 Index = 0; Index < Specs.Num(); Index++)
		{
			OutTestCommands.Push(Specs[Index]->Id);
			OutBeautifiedNames.Push(Specs[Index]->Description);
		}
	}

	void xDescribe(const FString& InDescription, TFunction<void()> DoWork)
	{
		//disabled
	}

	void xDescribe(const TArray<FString> InDescriptions, TFunction<void(int32 /*Iteration*/)> DoWork)
	{
		//disabled
	}

	void Describe(const TArray<FString> InDescriptions, TFunction<void(int32 /*Iteration*/)> DoWork)
	{
		LLM_SCOPE_BYNAME(TEXT("AutomationTest/Framework"));

		for (int32 It = 0; It < InDescriptions.Num(); ++It)
		{
			const TSharedRef<FSpecDefinitionScope> ParentScope = DefinitionScopeStack.Last();
			const TSharedRef<FSpecDefinitionScope> NewScope = MakeShareable(new FSpecDefinitionScope());
			NewScope->Description = InDescriptions[It];
			ParentScope->Children.Push(NewScope);

			DefinitionScopeStack.Push(NewScope);
			PushDescription(InDescriptions[It]);
			DoWork(It);
			PopDescription(InDescriptions[It]);
			DefinitionScopeStack.Pop();

			if (NewScope->It.Num() == 0 && NewScope->Children.Num() == 0)
			{
				ParentScope->Children.Remove(NewScope);
			}
		}
	}

	void Describe(const FString& InDescription, TFunction<void()> DoWork)
	{
		Describe({InDescription}, [DoWork](int32){DoWork();});
	}

	void xIt(const FString& InDescription, TFunction<void()> DoWork)
	{
		//disabled
	}

	void xIt(const FString& InDescription, EAsyncExecution Execution, TFunction<void()> DoWork)
	{
		//disabled
	}

	void xIt(const FString& InDescription, EAsyncExecution Execution, const FTimespan& Timeout, TFunction<void()> DoWork)
	{
		//disabled
	}

	void xLatentIt(const FString& InDescription, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		//disabled
	}

	void xLatentIt(const FString& InDescription, const FTimespan& Timeout, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		//disabled
	}

	void xLatentIt(const FString& InDescription, EAsyncExecution Execution, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		//disabled
	}

	void xLatentIt(const FString& InDescription, EAsyncExecution Execution, const FTimespan& Timeout, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		//disabled
	}

	void It(const FString& InDescription, TFunction<void()> DoWork, UE::FSourceLocation Location = UE::FSourceLocation::Current())
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();

		PushDescription(InDescription);
		CurrentScope->It.Push(MakeShareable(new FSpecIt(GetDescription(), GetId(), Location.GetFileName(), Location.GetLine(), MakeShareable(new FSingleExecuteLatentCommand(this, DoWork, bEnableSkipIfError)))));
		PopDescription(InDescription);
	}

	void It(const FString& InDescription, EAsyncExecution Execution, TFunction<void()> DoWork, UE::FSourceLocation Location = UE::FSourceLocation::Current())
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();

		PushDescription(InDescription);
		CurrentScope->It.Push(MakeShareable(new FSpecIt(GetDescription(), GetId(), Location.GetFileName(), Location.GetLine(), MakeShareable(new FAsyncLatentCommand(this, Execution, DoWork, DefaultTimeout, bEnableSkipIfError)))));
		PopDescription(InDescription);
	}

	void It(const FString& InDescription, EAsyncExecution Execution, const FTimespan& Timeout, TFunction<void()> DoWork, UE::FSourceLocation Location = UE::FSourceLocation::Current())
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();

		PushDescription(InDescription);
		CurrentScope->It.Push(MakeShareable(new FSpecIt(GetDescription(), GetId(), Location.GetFileName(), Location.GetLine(), MakeShareable(new FAsyncLatentCommand(this, Execution, DoWork, Timeout, bEnableSkipIfError)))));
		PopDescription(InDescription);
	}

	void LatentIt(const FString& InDescription, TFunction<void(const FDoneDelegate&)> DoWork, UE::FSourceLocation Location = UE::FSourceLocation::Current())
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();

		PushDescription(InDescription);
		CurrentScope->It.Push(MakeShareable(new FSpecIt(GetDescription(), GetId(), Location.GetFileName(), Location.GetLine(), MakeShareable(new FUntilDoneLatentCommand(this, DoWork, DefaultTimeout, bEnableSkipIfError)))));
		PopDescription(InDescription);
	}

	void LatentIt(const FString& InDescription, const FTimespan& Timeout, TFunction<void(const FDoneDelegate&)> DoWork, UE::FSourceLocation Location = UE::FSourceLocation::Current())
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();

		PushDescription(InDescription);
		CurrentScope->It.Push(MakeShareable(new FSpecIt(GetDescription(), GetId(), Location.GetFileName(), Location.GetLine(), MakeShareable(new FUntilDoneLatentCommand(this, DoWork, Timeout, bEnableSkipIfError)))));
		PopDescription(InDescription);
	}

	void LatentIt(const FString& InDescription, EAsyncExecution Execution, TFunction<void(const FDoneDelegate&)> DoWork, UE::FSourceLocation Location = UE::FSourceLocation::Current())
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();

		PushDescription(InDescription);
		CurrentScope->It.Push(MakeShareable(new FSpecIt(GetDescription(), GetId(), Location.GetFileName(), Location.GetLine(), MakeShareable(new FAsyncUntilDoneLatentCommand(this, Execution, DoWork, DefaultTimeout, bEnableSkipIfError)))));
		PopDescription(InDescription);
	}

	void LatentIt(const FString& InDescription, EAsyncExecution Execution, const FTimespan& Timeout, TFunction<void(const FDoneDelegate&)> DoWork, UE::FSourceLocation Location = UE::FSourceLocation::Current())
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();

		PushDescription(InDescription);
		CurrentScope->It.Push(MakeShareable(new FSpecIt(GetDescription(), GetId(), Location.GetFileName(), Location.GetLine(), MakeShareable(new FAsyncUntilDoneLatentCommand(this, Execution, DoWork, Timeout, bEnableSkipIfError)))));
		PopDescription(InDescription);
	}

	void xBeforeEach(TFunction<void()> DoWork)
	{
		// disabled
	}

	void xBeforeEach(EAsyncExecution Execution, TFunction<void()> DoWork)
	{
		// disabled
	}

	void xBeforeEach(EAsyncExecution Execution, const FTimespan& Timeout, TFunction<void()> DoWork)
	{
		// disabled
	}

	void xLatentBeforeEach(TFunction<void(const FDoneDelegate&)> DoWork)
	{
		// disabled
	}

	void xLatentBeforeEach(const FTimespan& Timeout, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		// disabled
	}

	void xLatentBeforeEach(EAsyncExecution Execution, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		// disabled
	}

	void xLatentBeforeEach(EAsyncExecution Execution, const FTimespan& Timeout, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		// disabled
	}

	void BeforeEach(TFunction<void()> DoWork)
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();
		CurrentScope->BeforeEach.Push(MakeShareable(new FSingleExecuteLatentCommand(this, DoWork, bEnableSkipIfError)));
	}

	void BeforeEach(EAsyncExecution Execution, TFunction<void()> DoWork)
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();
		CurrentScope->BeforeEach.Push(MakeShareable(new FAsyncLatentCommand(this, Execution, DoWork, DefaultTimeout, bEnableSkipIfError)));
	}

	void BeforeEach(EAsyncExecution Execution, const FTimespan& Timeout, TFunction<void()> DoWork)
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();
		CurrentScope->BeforeEach.Push(MakeShareable(new FAsyncLatentCommand(this, Execution, DoWork, Timeout, bEnableSkipIfError)));
	}

	void LatentBeforeEach(TFunction<void(const FDoneDelegate&)> DoWork)
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();
		CurrentScope->BeforeEach.Push(MakeShareable(new FUntilDoneLatentCommand(this, DoWork, DefaultTimeout, bEnableSkipIfError)));
	}

	void LatentBeforeEach(const FTimespan& Timeout, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();
		CurrentScope->BeforeEach.Push(MakeShareable(new FUntilDoneLatentCommand(this, DoWork, Timeout, bEnableSkipIfError)));
	}
	     
	void LatentBeforeEach(EAsyncExecution Execution, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();
		CurrentScope->BeforeEach.Push(MakeShareable(new FAsyncUntilDoneLatentCommand(this, Execution, DoWork, DefaultTimeout, bEnableSkipIfError)));
	}

	void LatentBeforeEach(EAsyncExecution Execution, const FTimespan& Timeout, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();
		CurrentScope->BeforeEach.Push(MakeShareable(new FAsyncUntilDoneLatentCommand(this, Execution, DoWork, Timeout, bEnableSkipIfError)));
	}

	void xAfterEach(TFunction<void()> DoWork)
	{
		// disabled
	}

	void xAfterEach(EAsyncExecution Execution, TFunction<void()> DoWork)
	{
		// disabled
	}

	void xAfterEach(EAsyncExecution Execution, const FTimespan& Timeout, TFunction<void()> DoWork)
	{
		// disabled
	}

	void xLatentAfterEach(TFunction<void(const FDoneDelegate&)> DoWork)
	{
		// disabled
	}

	void xLatentAfterEach(const FTimespan& Timeout, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		// disabled
	}

	void xLatentAfterEach(EAsyncExecution Execution, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		// disabled
	}

	void xLatentAfterEach(EAsyncExecution Execution, const FTimespan& Timeout, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		// disabled
	}

	void AfterEach(TFunction<void()> DoWork)
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();
		CurrentScope->AfterEach.Push(MakeShareable(new FSingleExecuteLatentCommand(this, DoWork)));
	}

	void AfterEach(EAsyncExecution Execution, TFunction<void()> DoWork)
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();
		CurrentScope->AfterEach.Push(MakeShareable(new FAsyncLatentCommand(this, Execution, DoWork, DefaultTimeout)));
	}

	void AfterEach(EAsyncExecution Execution, const FTimespan& Timeout, TFunction<void()> DoWork)
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();
		CurrentScope->AfterEach.Push(MakeShareable(new FAsyncLatentCommand(this, Execution, DoWork, Timeout)));
	}

	void LatentAfterEach(TFunction<void(const FDoneDelegate&)> DoWork)
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();
		CurrentScope->AfterEach.Push(MakeShareable(new FUntilDoneLatentCommand(this, DoWork, DefaultTimeout)));
	}

	void LatentAfterEach(const FTimespan& Timeout, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();
		CurrentScope->AfterEach.Push(MakeShareable(new FUntilDoneLatentCommand(this, DoWork, Timeout)));
	}

	void LatentAfterEach(EAsyncExecution Execution, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();
		CurrentScope->AfterEach.Push(MakeShareable(new FAsyncUntilDoneLatentCommand(this, Execution, DoWork, DefaultTimeout)));
	}

	void LatentAfterEach(EAsyncExecution Execution, const FTimespan& Timeout, TFunction<void(const FDoneDelegate&)> DoWork)
	{
		const TSharedRef<FSpecDefinitionScope> CurrentScope = DefinitionScopeStack.Last();
		CurrentScope->AfterEach.Push(MakeShareable(new FAsyncUntilDoneLatentCommand(this, Execution, DoWork, Timeout)));
	}

protected:

	/* The timespan for how long a block should be allowed to execute before giving up and failing the test */
	FTimespan DefaultTimeout;

	/* Whether or not BeforeEach and It blocks should skip execution if the test has already failed */
	bool bEnableSkipIfError;

	void EnsureDefinitions() const
	{
		if (!bHasBeenDefined)
		{
			const_cast<FAutomationSpecBase*>(this)->Define();
			const_cast<FAutomationSpecBase*>(this)->PostDefine();
		}
	}

	virtual void Define() = 0;

	void PostDefine()
	{
		TArray<TSharedRef<FSpecDefinitionScope>> Stack;
		Stack.Push(RootDefinitionScope.ToSharedRef());

		TArray<TSharedRef<IAutomationLatentCommand>> BeforeEach;
		TArray<TSharedRef<IAutomationLatentCommand>> AfterEach;

		while (Stack.Num() > 0)
		{
			const TSharedRef<FSpecDefinitionScope> Scope = Stack.Last();

			BeforeEach.Append(Scope->BeforeEach);
			AfterEach.Append(Scope->AfterEach); // iterate in reverse

			for (int32 ItIndex = 0; ItIndex < Scope->It.Num(); ItIndex++)
			{
				TSharedRef<FSpecIt> It = Scope->It[ItIndex];

				TSharedRef<FSpec> Spec = MakeShareable(new FSpec());
				Spec->Id = It->Id;
				Spec->Description = It->Description;
				Spec->Filename = It->Filename;
				Spec->LineNumber = It->LineNumber;
				Spec->Commands.Append(BeforeEach);
				Spec->Commands.Add(It->Command);

				for (int32 AfterEachIndex = AfterEach.Num() - 1; AfterEachIndex >= 0; AfterEachIndex--)
				{
					Spec->Commands.Add(AfterEach[AfterEachIndex]);
				}

				check(!IdToSpecMap.Contains(Spec->Id));
				IdToSpecMap.Add(Spec->Id, Spec);
			}
			Scope->It.Empty();

			if (Scope->Children.Num() > 0)
			{
				Stack.Append(Scope->Children);
				Scope->Children.Empty();
			}
			else
			{
				while (Stack.Num() > 0 && Stack.Last()->Children.Num() == 0 && Stack.Last()->It.Num() == 0)
				{
					const TSharedRef<FSpecDefinitionScope> PoppedScope = Stack.Pop();

					if (PoppedScope->BeforeEach.Num() > 0)
					{
						BeforeEach.RemoveAt(BeforeEach.Num() - PoppedScope->BeforeEach.Num(), PoppedScope->BeforeEach.Num());
					}

					if (PoppedScope->AfterEach.Num() > 0)
					{
						AfterEach.RemoveAt(AfterEach.Num() - PoppedScope->AfterEach.Num(), PoppedScope->AfterEach.Num());
					}
				}
			}
		}

		RootDefinitionScope.Reset();
		DefinitionScopeStack.Reset();
		bHasBeenDefined = true;
	}

	void Redefine()
	{
		Description.Empty();
		IdToSpecMap.Empty();
		RootDefinitionScope.Reset();
		DefinitionScopeStack.Empty();
		bHasBeenDefined = false;

		RootDefinitionScope = MakeShareable(new FSpecDefinitionScope());
		DefinitionScopeStack.Push(RootDefinitionScope.ToSharedRef());
	}

private:

	void PushDescription(const FString& InDescription)
	{
		LLM_SCOPE_BYNAME(TEXT("AutomationTest/Framework"));
		Description.Add(InDescription);
	}

	void PopDescription(const FString& InDescription)
	{
		Description.RemoveAt(Description.Num() - 1);
	}

	FString GetDescription() const
	{
		FString CompleteDescription;
		for (int32 Index = 0; Index < Description.Num(); ++Index)
		{
			if (Description[Index].IsEmpty())
			{
				continue;
			}

			if (CompleteDescription.IsEmpty())
			{
				CompleteDescription = Description[Index];
			}
			else if (FChar::IsWhitespace(CompleteDescription[CompleteDescription.Len() - 1]) || FChar::IsWhitespace(Description[Index][0]))
			{
				CompleteDescription = CompleteDescription + TEXT(".") + Description[Index];
			}
			else
			{
				CompleteDescription = FString::Printf(TEXT("%s.%s"), *CompleteDescription, *Description[Index]);
			}
		}

		return CompleteDescription;
	}

	FString GetId() const
	{
		if (Description.Last().EndsWith(TEXT("]")))
		{
			FString ItDescription = Description.Last();
			ItDescription.RemoveAt(ItDescription.Len() - 1);

			int32 StartingBraceIndex = INDEX_NONE;
			if (ItDescription.FindLastChar(TEXT('['), StartingBraceIndex) && StartingBraceIndex != ItDescription.Len() - 1)
			{
				FString CommandId = ItDescription.RightChop(StartingBraceIndex + 1);
				return CommandId;
			}
		}

		FString CompleteId;
		for (int32 Index = 0; Index < Description.Num(); ++Index)
		{
			if (Description[Index].IsEmpty())
			{
				continue;
			}

			if (CompleteId.IsEmpty())
			{
				CompleteId = Description[Index];
			}
			else if (FChar::IsWhitespace(CompleteId[CompleteId.Len() - 1]) || FChar::IsWhitespace(Description[Index][0]))
			{
				CompleteId = CompleteId + Description[Index];
			}
			else
			{
				CompleteId = FString::Printf(TEXT("%s %s"), *CompleteId, *Description[Index]);
			}
		}

		return CompleteId;
	}

private:
	TArray<FString> Description;
	TMap<FString, TSharedRef<FSpec>> IdToSpecMap;
	TSharedPtr<FSpecDefinitionScope> RootDefinitionScope;
	TArray<TSharedRef<FSpecDefinitionScope>> DefinitionScopeStack;
	bool bHasBeenDefined;
};


//////////////////////////////////////////////////
// Latent command definition macros

#define DEFINE_LATENT_AUTOMATION_COMMAND(CommandName)	\
class CommandName : public IAutomationLatentCommand \
	{ \
	public: \
	virtual ~CommandName() \
		{} \
		virtual bool Update() override; \
}

#define DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(CommandName,ParamType,ParamName)	\
class CommandName : public IAutomationLatentCommand \
	{ \
	public: \
	CommandName(ParamType InputParam) \
	: ParamName(InputParam) \
		{} \
		virtual ~CommandName() \
		{} \
		virtual bool Update() override; \
	private: \
	ParamType ParamName; \
}

#define DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(CommandName,ParamType0,ParamName0,ParamType1,ParamName1)	\
class CommandName : public IAutomationLatentCommand \
	{ \
	public: \
	CommandName(ParamType0 InputParam0, ParamType1 InputParam1) \
	: ParamName0(InputParam0) \
	, ParamName1(InputParam1) \
		{} \
		virtual ~CommandName() \
		{} \
		virtual bool Update() override; \
	private: \
	ParamType0 ParamName0; \
	ParamType1 ParamName1; \
}

#define DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(CommandName,ParamType0,ParamName0,ParamType1,ParamName1,ParamType2,ParamName2)	\
class CommandName : public IAutomationLatentCommand \
	{ \
	public: \
		CommandName(ParamType0 InputParam0, ParamType1 InputParam1, ParamType2 InputParam2) \
		: ParamName0(InputParam0) \
		, ParamName1(InputParam1) \
		, ParamName2(InputParam2) \
		{} \
		virtual ~CommandName() \
		{} \
		virtual bool Update() override; \
	private: \
	ParamType0 ParamName0; \
	ParamType1 ParamName1; \
	ParamType2 ParamName2; \
}

#define DEFINE_LATENT_AUTOMATION_COMMAND_FOUR_PARAMETER(CommandName,ParamType0,ParamName0,ParamType1,ParamName1,ParamType2,ParamName2,ParamType3,ParamName3)	\
class CommandName : public IAutomationLatentCommand \
	{ \
	public: \
		CommandName(ParamType0 InputParam0, ParamType1 InputParam1, ParamType2 InputParam2, ParamType3 InputParam3) \
		: ParamName0(InputParam0) \
		, ParamName1(InputParam1) \
		, ParamName2(InputParam2) \
		, ParamName3(InputParam3) \
		{} \
		virtual ~CommandName() \
		{} \
		virtual bool Update() override; \
	private: \
	ParamType0 ParamName0; \
	ParamType1 ParamName1; \
	ParamType2 ParamName2; \
	ParamType3 ParamName3; \
}

#define DEFINE_LATENT_AUTOMATION_COMMAND_FIVE_PARAMETER(CommandName,ParamType0,ParamName0,ParamType1,ParamName1,ParamType2,ParamName2,ParamType3,ParamName3,ParamType4,ParamName4)	\
class CommandName : public IAutomationLatentCommand \
	{ \
	public: \
		CommandName(ParamType0 InputParam0, ParamType1 InputParam1, ParamType2 InputParam2, ParamType3 InputParam3, ParamType4 InputParam4) \
		: ParamName0(InputParam0) \
		, ParamName1(InputParam1) \
		, ParamName2(InputParam2) \
		, ParamName3(InputParam3) \
		, ParamName4(InputParam4) \
		{} \
		virtual ~CommandName() \
		{} \
		virtual bool Update() override; \
	private: \
	ParamType0 ParamName0; \
	ParamType1 ParamName1; \
	ParamType2 ParamName2; \
	ParamType3 ParamName3; \
	ParamType4 ParamName4; \
}

#define DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND(EXPORT_API, CommandName)	\
class CommandName : public IAutomationLatentCommand \
	{ \
	public: \
	virtual ~CommandName() \
		{} \
		EXPORT_API virtual bool Update() override; \
}

#define DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(EXPORT_API, CommandName,ParamType,ParamName)	\
class CommandName : public IAutomationLatentCommand \
	{ \
	public: \
	CommandName(ParamType InputParam) \
	: ParamName(InputParam) \
		{} \
		virtual ~CommandName() \
		{} \
		EXPORT_API virtual bool Update() override; \
	private: \
	ParamType ParamName; \
}

#define DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(EXPORT_API, CommandName,ParamType0,ParamName0,ParamType1,ParamName1)	\
class CommandName : public IAutomationLatentCommand \
	{ \
	public: \
	CommandName(ParamType0 InputParam0, ParamType1 InputParam1) \
	: ParamName0(InputParam0) \
	, ParamName1(InputParam1) \
		{} \
		virtual ~CommandName() \
		{} \
		EXPORT_API virtual bool Update() override; \
	private: \
	ParamType0 ParamName0; \
	ParamType1 ParamName1; \
}

#define DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(EXPORT_API, CommandName,ParamType0,ParamName0,ParamType1,ParamName1,ParamType2,ParamName2)	\
class CommandName : public IAutomationLatentCommand \
	{ \
	public: \
	CommandName(ParamType0 InputParam0, ParamType1 InputParam1, ParamType2 InputParam2) \
	: ParamName0(InputParam0) \
	, ParamName1(InputParam1) \
	, ParamName2(InputParam2) \
		{} \
		virtual ~CommandName() \
		{} \
		EXPORT_API virtual bool Update() override; \
	private: \
	ParamType0 ParamName0; \
	ParamType1 ParamName1; \
	ParamType2 ParamName2; \
}

#define DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_FOUR_PARAMETER(EXPORT_API, CommandName,ParamType0,ParamName0,ParamType1,ParamName1,ParamType2,ParamName2,ParamType3,ParamName3)	\
class CommandName : public IAutomationLatentCommand \
	{ \
	public: \
	CommandName(ParamType0 InputParam0, ParamType1 InputParam1, ParamType2 InputParam2, ParamType3 InputParam3) \
	: ParamName0(InputParam0) \
	, ParamName1(InputParam1) \
	, ParamName2(InputParam2) \
	, ParamName3(InputParam3) \
		{} \
		virtual ~CommandName() \
		{} \
		EXPORT_API virtual bool Update() override; \
	private: \
	ParamType0 ParamName0; \
	ParamType1 ParamName1; \
	ParamType2 ParamName2; \
	ParamType3 ParamName3; \
}

#define DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_FIVE_PARAMETER(EXPORT_API, CommandName,ParamType0,ParamName0,ParamType1,ParamName1,ParamType2,ParamName2,ParamType3,ParamName3,ParamType4,ParamName4)	\
class CommandName : public IAutomationLatentCommand \
	{ \
	public: \
	CommandName(ParamType0 InputParam0, ParamType1 InputParam1, ParamType2 InputParam2, ParamType3 InputParam3, ParamType4 InputParam4) \
	: ParamName0(InputParam0) \
	, ParamName1(InputParam1) \
	, ParamName2(InputParam2) \
	, ParamName3(InputParam3) \
	, ParamName4(InputParam4) \
		{} \
		virtual ~CommandName() \
		{} \
		EXPORT_API virtual bool Update() override; \
	private: \
	ParamType0 ParamName0; \
	ParamType1 ParamName1; \
	ParamType2 ParamName2; \
	ParamType3 ParamName3; \
	ParamType4 ParamName4; \
}

#define DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND(CommandName)	\
	DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND(ENGINE_API, CommandName)

#define DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(CommandName,ParamType,ParamName)	\
	DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(ENGINE_API, CommandName, ParamType, ParamName)

#define DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_WITH_RETRIES(EXPORT_API,CommandName,RetryCount,WaitTimeBetweenRuns)	\
class CommandName : public IAutomationLatentCommandWithRetriesAndDelays \
	{ \
	public: \
	CommandName(int32 InRetryCount, double InWaitTimeBetweenRuns) \
	: IAutomationLatentCommandWithRetriesAndDelays(#CommandName, InRetryCount, InWaitTimeBetweenRuns) \
		{} \
		virtual ~CommandName() \
		{} \
		EXPORT_API virtual bool Execute() override; \
	private: \
}

#define DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_WITH_RETRIES_ONE_PARAMETER(EXPORT_API,CommandName,RetryCount,WaitTimeBetweenRuns,ParamType,ParamName)	\
class CommandName : public IAutomationLatentCommandWithRetriesAndDelays \
	{ \
	public: \
	CommandName(int32 InRetryCount, double InWaitTimeBetweenRuns, ParamType ParamName) \
	: IAutomationLatentCommandWithRetriesAndDelays(#CommandName, InRetryCount, InWaitTimeBetweenRuns) \
	, ParamName(ParamName) \
		{} \
		virtual ~CommandName() \
		{} \
		EXPORT_API virtual bool Execute() override; \
	private: \
	ParamType ParamName; \
}

#define DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_WITH_RETRIES_TWO_PARAMETERS(EXPORT_API,CommandName,RetryCount,WaitTimeBetweenRuns,ParamType0,ParamName0,ParamType1,ParamName1)	\
class CommandName : public IAutomationLatentCommandWithRetriesAndDelays \
	{ \
	public: \
	CommandName(int32 InRetryCount, double InWaitTimeBetweenRuns, ParamType0 ParamName0, ParamType1 ParamName1) \
	: IAutomationLatentCommandWithRetriesAndDelays(#CommandName, InRetryCount, InWaitTimeBetweenRuns) \
	, ParamName0(ParamName0) \
	, ParamName1(ParamName1) \
		{} \
		virtual ~CommandName() \
		{} \
		EXPORT_API virtual bool Execute() override; \
	private: \
	ParamType0 ParamName0; \
	ParamType1 ParamName1; \
}

#define DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_WITH_RETRIES_THREE_PARAMETERS(EXPORT_API,CommandName,RetryCount,WaitTimeBetweenRuns,ParamType0,ParamName0,ParamType1,ParamName1,ParamType2,ParamName2)	\
class CommandName : public IAutomationLatentCommandWithRetriesAndDelays \
	{ \
	public: \
	CommandName(int32 InRetryCount, double InWaitTimeBetweenRuns, ParamType0 ParamName0, ParamType1 ParamName1, ParamType2 ParamName2) \
	: IAutomationLatentCommandWithRetriesAndDelays(#CommandName, InRetryCount, InWaitTimeBetweenRuns) \
	, ParamName0(ParamName0) \
	, ParamName1(ParamName1) \
	, ParamName2(ParamName2) \
		{} \
		virtual ~CommandName() \
		{} \
		EXPORT_API virtual bool Execute() override; \
	private: \
	ParamType0 ParamName0; \
	ParamType1 ParamName1; \
	ParamType2 ParamName2; \
}

#define DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_WITH_RETRIES_FOUR_PARAMETERS(EXPORT_API,CommandName,RetryCount,WaitTimeBetweenRuns,ParamType0,ParamName0,ParamType1,ParamName1,ParamType2,ParamName2,ParamType3,ParamName3)	\
class CommandName : public IAutomationLatentCommandWithRetriesAndDelays \
	{ \
	public: \
	CommandName(int32 InRetryCount, double InWaitTimeBetweenRuns, ParamType0 ParamName0, ParamType1 ParamName1, ParamType2 ParamName2, ParamType3 ParamName3) \
	: IAutomationLatentCommandWithRetriesAndDelays(#CommandName, InRetryCount, InWaitTimeBetweenRuns) \
	, ParamName0(ParamName0) \
	, ParamName1(ParamName1) \
	, ParamName2(ParamName2) \
	, ParamName3(ParamName3) \
		{} \
		virtual ~CommandName() \
		{} \
		EXPORT_API virtual bool Execute() override; \
	private: \
	ParamType0 ParamName0; \
	ParamType1 ParamName1; \
	ParamType2 ParamName2; \
	ParamType3 ParamName3; \
}

//macro to simply the syntax for enqueueing a latent command
#if WITH_AUTOMATION_TESTS
#define ADD_LATENT_AUTOMATION_COMMAND(ClassDeclaration) FAutomationTestFramework::Get().EnqueueLatentCommand(MakeShareable(new ClassDeclaration));
#else
#define ADD_LATENT_AUTOMATION_COMMAND(ClassDeclaration) 
#endif

//declare the class
#define START_NETWORK_AUTOMATION_COMMAND(ClassDeclaration)	\
class F##ClassDeclaration : public IAutomationNetworkCommand \
{ \
private:\
	int32 RoleIndex; \
public: \
	F##ClassDeclaration(int32 InRoleIndex) : RoleIndex(InRoleIndex) {} \
	virtual ~F##ClassDeclaration() {} \
	virtual uint32 GetRoleIndex() const override { return RoleIndex; } \
	virtual void Run() override 

//close the class and add to the framework
#define END_NETWORK_AUTOMATION_COMMAND(ClassDeclaration,InRoleIndex) }; \
	FAutomationTestFramework::Get().EnqueueNetworkCommand(MakeShareable(new F##ClassDeclaration(InRoleIndex))); \

/**
 * Macros to simplify the creation of new automation tests. To create a new test one simply must put
 * IMPLEMENT_SIMPLE_AUTOMATION_TEST( NewAutomationClassName, AutomationClassFlags )
 * IMPLEMENT_COMPLEX_AUTOMATION_TEST( NewAutomationClassName, AutomationClassFlags )
 * in their cpp file, and then proceed to write an implementation for:
 * bool NewAutomationTestClassName::RunTest() {}
 * While the macro could also have allowed the code to be specified, leaving it out of the macro allows
 * the code to be debugged more easily.
 *
 * Builds supporting automation tests will automatically create and register an instance of the automation test within
 * the automation test framework as a result of the macro.
 */

#define IMPLEMENT_SIMPLE_AUTOMATION_TEST_PRIVATE( TClass, TBaseClass, PrettyName, TFlags, FileName, LineNumber ) \
	class TClass : public TBaseClass \
	{ \
	public: \
		TClass( const FString& InName ) \
		:TBaseClass( InName, false ) {\
			static_assert(!!((TFlags) & EAutomationTestFlags_ApplicationContextMask), "AutomationTest has no application flag.  It shouldn't run.  See AutomationTest.h."); \
			static_assert(	!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::SmokeFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::EngineFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::ProductFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::PerfFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::StressFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::NegativeFilter), \
							"All AutomationTests must have exactly 1 filter type specified.  See AutomationTest.h."); \
		} \
		virtual EAutomationTestFlags GetTestFlags() const override { return TFlags; } \
		virtual bool IsStressTest() const { return false; } \
		virtual uint32 GetRequiredDeviceNum() const override { return 1; } \
		virtual FString GetTestSourceFileName() const override { return FileName; } \
		virtual int32 GetTestSourceFileLine() const override { return LineNumber; } \
	protected: \
		virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const override \
		{ \
			OutBeautifiedNames.Add(PrettyName); \
			OutTestCommands.Add(FString()); \
		} \
		AUTORTFM_DISABLE_IF(!EnumHasAnyFlags(TFlags, EAutomationTestFlags::SupportsAutoRTFM)) \
		virtual bool RunTest(const FString& Parameters) override; \
		virtual FString GetBeautifiedTestName() const override { return PrettyName; } \
	};



#define IMPLEMENT_COMPLEX_AUTOMATION_TEST_PRIVATE( TClass, TBaseClass, PrettyName, TFlags, FileName, LineNumber ) \
	class TClass : public TBaseClass \
	{ \
	public: \
		TClass( const FString& InName ) \
		:TBaseClass( InName, true ) { \
			static_assert(!!((TFlags) & EAutomationTestFlags_ApplicationContextMask), "AutomationTest has no application flag.  It shouldn't run.  See AutomationTest.h."); \
			static_assert(	!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::SmokeFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::EngineFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::ProductFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::PerfFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::StressFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::NegativeFilter), \
							"All AutomationTests must have exactly 1 filter type specified.  See AutomationTest.h."); \
		} \
		virtual EAutomationTestFlags GetTestFlags() const override { return ((TFlags) & ~(EAutomationTestFlags::SmokeFilter)); } \
		virtual bool IsStressTest() const { return true; } \
		virtual uint32 GetRequiredDeviceNum() const override { return 1; } \
		virtual FString GetTestSourceFileName() const override { return FileName; } \
		virtual int32 GetTestSourceFileLine() const override { return LineNumber; } \
	protected: \
		virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const override; \
		AUTORTFM_DISABLE_IF(!EnumHasAnyFlags(TFlags, EAutomationTestFlags::SupportsAutoRTFM)) \
		virtual bool RunTest(const FString& Parameters) override; \
		virtual FString GetBeautifiedTestName() const override { return PrettyName; } \
	};

#define IMPLEMENT_NETWORKED_AUTOMATION_TEST_PRIVATE(TClass, TBaseClass, PrettyName, TFlags, NumParticipants, FileName, LineNumber) \
	class TClass : public TBaseClass \
	{ \
	public: \
		TClass( const FString& InName ) \
		:TBaseClass( InName, false ) { \
			static_assert(!!((TFlags) & EAutomationTestFlags_ApplicationContextMask), "AutomationTest has no application flag.  It shouldn't run.  See AutomationTest.h."); \
			static_assert(	!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::SmokeFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::EngineFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::ProductFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::PerfFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::StressFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::NegativeFilter), \
							"All AutomationTests must have exactly 1 filter type specified.  See AutomationTest.h."); \
		} \
		virtual EAutomationTestFlags GetTestFlags() const override { return ((TFlags) & ~(EAutomationTestFlags::EditorContext | EAutomationTestFlags::CommandletContext | EAutomationTestFlags::SmokeFilter)); } \
		virtual uint32 GetRequiredDeviceNum() const override { return NumParticipants; } \
		virtual FString GetTestSourceFileName() const override { return FileName; } \
		virtual int32 GetTestSourceFileLine() const override { return LineNumber; } \
	protected: \
		virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const override \
		{ \
			OutBeautifiedNames.Add(PrettyName); \
			OutTestCommands.Add(FString()); \
		} \
		AUTORTFM_DISABLE_IF(!EnumHasAnyFlags(TFlags, EAutomationTestFlags::SupportsAutoRTFM)) \
		virtual bool RunTest(const FString& Parameters) override; \
		virtual FString GetBeautifiedTestName() const override { return PrettyName; } \
	};

#define IMPLEMENT_BDD_AUTOMATION_TEST_PRIVATE( TClass, PrettyName, TFlags, FileName, LineNumber ) \
	class TClass : public FBDDAutomationTestBase \
	{ \
	public: \
		TClass( const FString& InName ) \
		:FBDDAutomationTestBase( InName, false ) {\
			static_assert(!!((TFlags) & EAutomationTestFlags_ApplicationContextMask), "AutomationTest has no application flag.  It shouldn't run.  See AutomationTest.h."); \
			static_assert(	!!((((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::SmokeFilter) || \
							!!((((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::EngineFilter) || \
							!!((((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::ProductFilter) || \
							!!((((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::PerfFilter) || \
							!!((((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::StressFilter) || \
							!!((((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::NegativeFilter), \
							"All AutomationTests must have exactly 1 filter type specified.  See AutomationTest.h."); \
		} \
		virtual EAutomationTestFlags GetTestFlags() const override { return TFlags; } \
		virtual bool IsStressTest() const { return false; } \
		virtual uint32 GetRequiredDeviceNum() const override { return 1; } \
		virtual FString GetTestSourceFileName() const override { return FileName; } \
		virtual int32 GetTestSourceFileLine() const override { return LineNumber; } \
	protected: \
		AUTORTFM_DISABLE_IF(!EnumHasAnyFlags(TFlags, EAutomationTestFlags::SupportsAutoRTFM)) \
		virtual bool RunTest(const FString& Parameters) override; \
		virtual FString GetBeautifiedTestName() const override { return PrettyName; } \
	private: \
		void Define(); \
	};

#define DEFINE_SPEC_PRIVATE( TClass, PrettyName, TFlags, FileName, LineNumber ) \
	class TClass : public FAutomationSpecBase \
	{ \
	public: \
		TClass( const FString& InName ) \
		: FAutomationSpecBase( InName, false ) {\
			static_assert(!!((TFlags) & EAutomationTestFlags_ApplicationContextMask), "AutomationTest has no application flag.  It shouldn't run.  See AutomationTest.h."); \
			static_assert(	!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::SmokeFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::EngineFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::ProductFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::PerfFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::StressFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::NegativeFilter), \
							"All AutomationTests must have exactly 1 filter type specified.  See AutomationTest.h."); \
		} \
		virtual EAutomationTestFlags GetTestFlags() const override { return TFlags; } \
        using FAutomationSpecBase::GetTestSourceFileName; \
		virtual FString GetTestSourceFileName() const override { return FileName; } \
        using FAutomationSpecBase::GetTestSourceFileLine; \
		virtual int32 GetTestSourceFileLine() const override { return LineNumber; } \
		virtual FString GetTestSourceFileName(const FString&) const override { return GetTestSourceFileName(); } \
		virtual int32 GetTestSourceFileLine(const FString&) const override { return GetTestSourceFileLine(); } \
	protected: \
		virtual FString GetBeautifiedTestName() const override { return PrettyName; } \
		virtual void Define() override; \
	};

#define BEGIN_DEFINE_SPEC_PRIVATE( TClass, PrettyName, TFlags, FileName, LineNumber ) \
	class AUTORTFM_DISABLE_IF(!EnumHasAnyFlags(TFlags, EAutomationTestFlags::SupportsAutoRTFM)) TClass : public FAutomationSpecBase \
	{ \
	public: \
		TClass( const FString& InName ) \
		: FAutomationSpecBase( InName, false ) {\
			static_assert(!!((TFlags) & EAutomationTestFlags_ApplicationContextMask), "AutomationTest has no application flag.  It shouldn't run.  See AutomationTest.h."); \
			static_assert(	!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::SmokeFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::EngineFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::ProductFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::PerfFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::StressFilter) || \
							!!(((TFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::NegativeFilter), \
							"All AutomationTests must have exactly 1 filter type specified.  See AutomationTest.h."); \
		} \
		virtual EAutomationTestFlags GetTestFlags() const override { return TFlags; } \
		using FAutomationSpecBase::GetTestSourceFileName; \
		virtual FString GetTestSourceFileName() const override { return FileName; } \
		using FAutomationSpecBase::GetTestSourceFileLine; \
		virtual int32 GetTestSourceFileLine() const override { return LineNumber; } \
	protected: \
		virtual FString GetBeautifiedTestName() const override { return PrettyName; } \
		virtual void Define() override;

#define REGISTER_SIMPLE_AUTOMATION_TEST_TAGS( TClass, PrettyName, TagsString ) \
	namespace\
	{\
		FAutomationTestTags TClass##AutomationTagsInstance(PrettyName, TagsString);\
	}

#if WITH_AUTOMATION_WORKER
	#define IMPLEMENT_SIMPLE_AUTOMATION_TEST( TClass, PrettyName, TFlags ) \
		IMPLEMENT_SIMPLE_AUTOMATION_TEST_PRIVATE(TClass, FAutomationTestBase, PrettyName, TFlags, __FILE__, __LINE__) \
		namespace\
		{\
			TClass TClass##AutomationTestInstance( TEXT(#TClass) );\
		}
	#define IMPLEMENT_COMPLEX_AUTOMATION_TEST( TClass, PrettyName, TFlags ) \
		IMPLEMENT_COMPLEX_AUTOMATION_TEST_PRIVATE(TClass, FAutomationTestBase, PrettyName, TFlags, __FILE__, __LINE__) \
		namespace\
		{\
			TClass TClass##AutomationTestInstance( TEXT(#TClass) );\
		}
	#define IMPLEMENT_COMPLEX_AUTOMATION_CLASS( TClass, PrettyName, TFlags ) \
		IMPLEMENT_COMPLEX_AUTOMATION_TEST_PRIVATE(TClass, FAutomationTestBase, PrettyName, TFlags, __FILE__, __LINE__)
	#define IMPLEMENT_NETWORKED_AUTOMATION_TEST(TClass, PrettyName, TFlags, NumParticipants) \
		IMPLEMENT_NETWORKED_AUTOMATION_TEST_PRIVATE(TClass, FAutomationTestBase, PrettyName, TFlags, NumParticipants, __FILE__, __LINE__) \
		namespace\
		{\
			TClass TClass##AutomationTestInstance( TEXT(#TClass) );\
		}

	#define IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST( TClass, TBaseClass, PrettyName, TFlags ) \
		IMPLEMENT_SIMPLE_AUTOMATION_TEST_PRIVATE(TClass, TBaseClass, PrettyName, TFlags, __FILE__, __LINE__) \
		namespace\
		{\
			TClass TClass##AutomationTestInstance( TEXT(#TClass) );\
		}

	#define IMPLEMENT_CUSTOM_COMPLEX_AUTOMATION_TEST( TClass, TBaseClass, PrettyName, TFlags ) \
		IMPLEMENT_COMPLEX_AUTOMATION_TEST_PRIVATE(TClass, TBaseClass, PrettyName, TFlags, __FILE__, __LINE__) \
		namespace\
		{\
			TClass TClass##AutomationTestInstance( TEXT(#TClass) );\
		}

	#define IMPLEMENT_BDD_AUTOMATION_TEST( TClass, PrettyName, TFlags ) \
		IMPLEMENT_BDD_AUTOMATION_TEST_PRIVATE(TClass, PrettyName, TFlags, __FILE__, __LINE__) \
		namespace\
		{\
			TClass TClass##AutomationTestInstance( TEXT(#TClass) );\
		}

	#define DEFINE_SPEC( TClass, PrettyName, TFlags ) \
		DEFINE_SPEC_PRIVATE(TClass, PrettyName, TFlags, __FILE__, __LINE__) \
		namespace\
		{\
			TClass TClass##AutomationSpecInstance( TEXT(#TClass) );\
		}

	#define BEGIN_DEFINE_SPEC( TClass, PrettyName, TFlags ) \
		BEGIN_DEFINE_SPEC_PRIVATE(TClass, PrettyName, TFlags, __FILE__, __LINE__) 

	#define END_DEFINE_SPEC( TClass ) \
		};\
		namespace\
		{\
			TClass TClass##AutomationSpecInstance( TEXT(#TClass) );\
		}

	//#define BEGIN_CUSTOM_COMPLEX_AUTOMATION_TEST( TClass, TBaseClass, PrettyName, TFlags ) \
	//	BEGIN_COMPLEX_AUTOMATION_TEST_PRIVATE(TClass, TBaseClass, PrettyName, TFlags, __FILE__, __LINE__)
	//
	//#define END_CUSTOM_COMPLEX_AUTOMATION_TEST( TClass ) \
	//	BEGIN_COMPLEX_AUTOMATION_TEST_PRIVATE(TClass, TBaseClass, PrettyName, TFlags, __FILE__, __LINE__)
	//	namespace\
	//	{\
	//		TClass TClass##AutomationTestInstance( TEXT(#TClass) );\
	//	}

#else
	#define IMPLEMENT_SIMPLE_AUTOMATION_TEST( TClass, PrettyName, TFlags ) \
		IMPLEMENT_SIMPLE_AUTOMATION_TEST_PRIVATE(TClass, FAutomationTestBase, PrettyName, TFlags, __FILE__, __LINE__)
	#define IMPLEMENT_COMPLEX_AUTOMATION_TEST( TClass, PrettyName, TFlags ) \
		IMPLEMENT_COMPLEX_AUTOMATION_TEST_PRIVATE(TClass, FAutomationTestBase, PrettyName, TFlags, __FILE__, __LINE__)
	#define IMPLEMENT_NETWORKED_AUTOMATION_TEST(TClass, PrettyName, TFlags, NumParticipants) \
		IMPLEMENT_NETWORKED_AUTOMATION_TEST_PRIVATE(TClass, FAutomationTestBase, PrettyName, TFlags, NumParticipants, __FILE__, __LINE__)

	#define IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST( TClass, TBaseClass, PrettyName, TFlags ) \
		IMPLEMENT_SIMPLE_AUTOMATION_TEST_PRIVATE(TClass, TBaseClass, PrettyName, TFlags, __FILE__, __LINE__)
	#define IMPLEMENT_CUSTOM_COMPLEX_AUTOMATION_TEST( TClass, TBaseClass, PrettyName, TFlags ) \
		IMPLEMENT_COMPLEX_AUTOMATION_TEST_PRIVATE(TClass, TBaseClass, PrettyName, TFlags, __FILE__, __LINE__)
	#define IMPLEMENT_BDD_AUTOMATION_TEST(TClass, PrettyName, TFlags) \
		IMPLEMENT_BDD_AUTOMATION_TEST_PRIVATE(TClass, PrettyName, TFlags, __FILE__, __LINE__)
	#define DEFINE_SPEC(TClass, PrettyName, TFlags) \
		DEFINE_SPEC_PRIVATE(TClass, PrettyName, TFlags, __FILE__, __LINE__)
	#define BEGIN_DEFINE_SPEC(TClass, PrettyName, TFlags) \
		BEGIN_DEFINE_SPEC_PRIVATE(TClass, PrettyName, TFlags, __FILE__, __LINE__)
	#define END_DEFINE_SPEC(TClass) \
		}; \

	//#define BEGIN_CUSTOM_COMPLEX_AUTOMATION_TEST( TClass, TBaseClass, PrettyName, TFlags ) \
	//	BEGIN_CUSTOM_COMPLEX_AUTOMATION_TEST_PRIVATE(TClass, TBaseClass, PrettyName, TFlags, __FILE__, __LINE__)

	//#define END_CUSTOM_COMPLEX_AUTOMATION_TEST( TClass )
	//	END_COMPLEX_AUTOMATION_TEST_PRIVATE(TClass, TBaseClass, PrettyName, TFlags, __FILE__, __LINE__)
#endif // #if WITH_AUTOMATION_WORKER


/**
 * Macros for early exit one-liners: they will run the appropriate
 * test method and on failure also execute `return false;`, which (if placed in
 * the main test case method) will stop the test immediately.
 *
 * The error logging is already handled by the test method being called.
 * EXPR variants automatically generate a `What` description string.
 *
 * As a result, you can easily test things that, if wrong, would potentially crash the test:
 *
 *		bool FMyEasyTest::RunTest(const FString& Parameters)
 *		{
 *			TArray<float> Data = GetSomeData();
 *			int32 Index = GetSomeIndex();
 *			UTEST_TRUE("Check valid index", Index < Data.Num());
 *			float DataItem = Data[Index];   // Won't crash, the test exited on the previous 
 *										    // line if index was invalid.
 *			UTEST_TRUE("Check valid item", DataItem > 0.f);
 *		}
 *
 */

#define UTEST_EQUAL(What, Actual, Expected)\
	if (!TestEqual(What, Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_EQUAL_EXPR(Actual, Expected)\
	if (!TestEqual(TEXT(#Actual), Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_EQUAL_TOLERANCE(What, Actual, Expected, Tolerance)\
	if (!TestEqual(What, Actual, Expected, Tolerance))\
	{\
		return false;\
	}

#define UTEST_EQUAL_TOLERANCE_EXPR(Actual, Expected, Tolerance)\
	if (!TestEqual(TEXT(#Actual), Actual, Expected, Tolerance))\
	{\
		return false;\
	}

#define UTEST_NEARLY_EQUAL(What, Actual, Expected, Tolerance)\
	if (!TestNearlyEqual(What, Actual, Expected, Tolerance))\
	{\
		return false;\
	}

#define UTEST_NEARLY_EQUAL_EXPR(Actual, Expected, Tolerance)\
	if (!TestNearlyEqual(TEXT(#Actual), Actual, Expected, Tolerance))\
	{\
		return false;\
	}

#define UTEST_EQUAL_INSENSITIVE(What, Actual, Expected)\
	if (!TestEqual(What, Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_EQUAL_INSENSITIVE_EXPR(Actual, Expected)\
	if (!TestEqual(TEXT(#Actual), Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_NOT_EQUAL_INSENSITIVE(What, Actual, Expected)\
	if (!TestNotEqual(What, Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_NOT_EQUAL_INSENSITIVE_EXPR(Actual, Expected)\
	if (!TestNotEqual(TEXT(#Actual), Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_EQUAL_SENSITIVE(What, Actual, Expected)\
	if (!TestEqualSensitive(What, Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_EQUAL_SENSITIVE_EXPR(Actual, Expected)\
	if (!TestEqualSensitive(TEXT(#Actual), Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_NOT_EQUAL_SENSITIVE(What, Actual, Expected)\
	if (!TestNotEqualSensitive(What, Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_NOT_EQUAL_SENSITIVE_EXPR(Actual, Expected)\
	if (!TestNotEqualSensitive(TEXT(#Actual), Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_NOT_EQUAL(What, Actual, Expected)\
	if (!TestNotEqual(What, Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_NOT_EQUAL_EXPR(Actual, Expected)\
	if (!TestNotEqual(FString::Printf(TEXT("%s != %s"), TEXT(#Actual), TEXT(#Expected)), Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_LESS(What, Actual, Expected)\
	if (!TestLessThan(What, Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_LESS_EXPR(Actual, Expected)\
	if (!TestLessThan(TEXT(#Actual), Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_LESS_TOLERANCE(What, Actual, Expected, Tolerance)\
	if (!TestLessThan(What, Actual, Expected, Tolerance))\
	{\
		return false;\
	}

#define UTEST_LESS_TOLERANCE_EXPR(Actual, Expected, Tolerance)\
	if (!TestLessThan(TEXT(#Actual), Actual, Expected, Tolerance))\
	{\
		return false;\
	}

#define UTEST_GREATER(What, Actual, Expected)\
	if (!TestGreaterThan(What, Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_GREATER_EXPR(Actual, Expected)\
	if (!TestGreaterThan(TEXT(#Actual), Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_GREATER_TOLERANCE(What, Actual, Expected, Tolerance)\
	if (!TestGreaterThan(What, Actual, Expected, Tolerance))\
	{\
		return false;\
	}

#define UTEST_GREATER_TOLERANCE_EXPR(Actual, Expected, Tolerance)\
	if (!TestGreaterThan(TEXT(#Actual), Actual, Expected, Tolerance))\
	{\
		return false;\
	}

#define UTEST_LESS_EQUAL(What, Actual, Expected)\
	if (!TestLessEqual(What, Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_LESS_EQUAL_EXPR(Actual, Expected)\
	if (!TestLessEqual(TEXT(#Actual), Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_LESS_EQUAL_TOLERANCE(What, Actual, Expected, Tolerance)\
	if (!TestLessEqual(What, Actual, Expected, Tolerance))\
	{\
		return false;\
	}

#define UTEST_LESS_EQUAL_TOLERANCE_EXPR(Actual, Expected, Tolerance)\
	if (!TestLessEqual(TEXT(#Actual), Actual, Expected, Tolerance))\
	{\
		return false;\
	}

#define UTEST_GREATER_EQUAL(What, Actual, Expected)\
	if (!TestGreaterEqual(What, Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_GREATER_EQUAL_EXPR(Actual, Expected)\
	if (!TestGreaterEqual(TEXT(#Actual), Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_GREATER_EQUAL_TOLERANCE(What, Actual, Expected, Tolerance)\
	if (!TestGreaterEqual(What, Actual, Expected, Tolerance))\
	{\
		return false;\
	}

#define UTEST_GREATER_EQUAL_TOLERANCE_EXPR(Actual, Expected, Tolerance)\
	if (!TestGreaterEqual(TEXT(#Actual), Actual, Expected, Tolerance))\
	{\
		return false;\
	}

#define UTEST_SAME(What, Actual, Expected)\
	if (!TestSame(What, Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_SAME_EXPR(Actual, Expected)\
	if (!TestSame(FString::Printf(TEXT("%s == %s"), TEXT(#Actual), TEXT(#Expected)), Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_NOT_SAME(What, Actual, Expected)\
	if (!TestNotSame(What, Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_NOT_SAME_EXPR(Actual, Expected)\
	if (!TestNotSame(FString::Printf(TEXT("%s != %s"), TEXT(#Actual), TEXT(#Expected)), Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_SAME_PTR(What, Actual, Expected)\
	if (!TestSamePtr(What, Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_SAME_PTR_EXPR(Actual, Expected)\
	if (!TestSamePtr(FString::Printf(TEXT("%s == %s"), TEXT(#Actual), TEXT(#Expected)), Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_NOT_SAME_PTR(What, Actual, Expected)\
	if (!TestNotSamePtr(What, Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_NOT_SAME_PTR_EXPR(Actual, Expected)\
	if (!TestNotSamePtr(FString::Printf(TEXT("%s != %s"), TEXT(#Actual), TEXT(#Expected)), Actual, Expected))\
	{\
		return false;\
	}

#define UTEST_TRUE(What, Value)\
	if (!TestTrue(What, Value))\
	{\
		return false;\
	}

#define UTEST_TRUE_EXPR(Expression)\
	if (!TestTrue(TEXT(#Expression), Expression))\
	{\
		return false;\
	}

#define UTEST_FALSE(What, Value)\
	if (!TestFalse(What, Value))\
	{\
		return false;\
	}

#define UTEST_FALSE_EXPR(Expression)\
	if (!TestFalse(TEXT(#Expression), Expression))\
	{\
		return false;\
	}

#define UTEST_VALID(What, Value)\
	if (!TestValid(What, Value))\
	{\
		return false;\
	}

#define UTEST_VALID_EXPR(Value)\
	if (!TestValid(TEXT(#Value), Value))\
	{\
		return false;\
	}

#define UTEST_INVALID(What, Value)\
	if (!TestInvalid(What, Value))\
	{\
		return false;\
	}

#define UTEST_INVALID_EXPR(Value)\
	if (!TestInvalid(TEXT(#Value), Value))\
	{\
		return false;\
	}

#define UTEST_NULL(What, Pointer)\
	if (!TestNull(What, Pointer))\
	{\
		return false;\
	}

#define UTEST_NULL_EXPR(Pointer)\
	if (!TestNull(TEXT(#Pointer), Pointer))\
	{\
		return false;\
	}

#define UTEST_NOT_NULL(What, Pointer)\
	if (!TestNotNull(What, Pointer))\
	{\
		return false;\
	}\
	CA_ASSUME(Pointer);

#define UTEST_NOT_NULL_EXPR(Pointer)\
	if (!TestNotNull(TEXT(#Pointer), Pointer))\
	{\
		return false;\
	}\
	CA_ASSUME(Pointer);

//////////////////////////////////////////////////
// Basic Latent Commands

/**
 * Run some code latently with a predicate lambda.  If the predicate returns false, the latent action will be called 
 * again next frame.  If it returns true, the command will stop running.
 */
class FFunctionLatentCommand : public IAutomationLatentCommand
{
public:
	FFunctionLatentCommand(TFunction<bool()> InLatentPredicate)
		: LatentPredicate(MoveTemp(InLatentPredicate))
	{
	}

	virtual ~FFunctionLatentCommand() = default;

	virtual bool Update() override
	{
		return LatentPredicate();
	}

private:
	TFunction<bool()> LatentPredicate;
};


class FDelayedFunctionLatentCommand : public IAutomationLatentCommand
{
public:
	FDelayedFunctionLatentCommand(TFunction<void()> InCallback, float InDelay = 0.1f)
		: Callback(MoveTemp(InCallback))
		, Delay(InDelay)
	{}

	virtual bool Update() override
	{
		double NewTime = FPlatformTime::Seconds();
		if ( NewTime - StartTime >= Delay )
		{
			Callback();
			return true;
		}
		return false;
	}

private:
	TFunction<void()> Callback;
	float Delay;
};


class FUntilCommand : public IAutomationLatentCommand
{
public:
	FUntilCommand(TFunction<bool()> InCallback, TFunction<bool()> InTimeoutCallback, float InTimeout = 5.0f)
		: Callback(MoveTemp(InCallback))
		, TimeoutCallback(MoveTemp(InTimeoutCallback))
		, Timeout(InTimeout)
	{}

	FUntilCommand(TFunction<bool()> InCallback, float InTimeout = 5.0f)
		: Callback(MoveTemp(InCallback))
		, TimeoutCallback([]() { return true; })
		, Timeout(InTimeout)
	{}

	virtual bool Update() override
	{
		if ( !Callback() )
		{
			const double NewTime = FPlatformTime::Seconds();
			if ( NewTime - StartTime >= Timeout )
			{
				TimeoutCallback();
				return true;
			}

			return false;
		}

		return true;
	}

private:
	TFunction<bool()> Callback;
	TFunction<bool()> TimeoutCallback;
	float Timeout;
};

// Extension of IAutomationLatentCommand with delays between attempts
// if initial command does not succeed. Has a max retry count that can be 
// overridden. Default is 10 retries with a 1 second delay in between.
// To protect against misuse of unlimited retries, has a Max execution 
// time of 5 minutes. This can be overridden, but not recommended. Only 
// do this if you're absolutely certain it will not cause an infinite loop
class IAutomationLatentCommandWithRetriesAndDelays : public IAutomationLatentCommand
{
public:
	virtual ~IAutomationLatentCommandWithRetriesAndDelays() = default;

	virtual void CommandFailedDueToError(const FString& ErrorMessage)
	{
		// Stop further commands and log error so the test fails.
		FAutomationTestFramework::Get().DequeueAllCommands();
		// Must log here so that Gauntlet can also pick up the error for its report. Otherwise, it will only show up in the Automation log.
		UE_LOGF(LogLatentCommands, Error, "%ls", *ErrorMessage);
		GetCurrentTest()->AddError(ErrorMessage);
	}

	// Base Update override with delay logic built in. Can be further overridden in child
	// classes
	virtual bool Update() override
	{
		if (bHasUnlimitedRetries)
		{
			// command has unlimited retries, so need to check if max run time has been exceeded
			if (HasExceededMaxTotalRunTime())
			{
				// Command run time has exceeded max total run time.
				FString ErrorMessageText = FString::Printf(
					TEXT("%s has failed due to exceeding the max allowed run time of %f seconds. ")
					TEXT("This may be due to an error, or having a single command attempt to do ")
					TEXT("too many things. If this is not due to an error, consider breaking ")
					TEXT("up this command into multiple, smaller commands."), *GetTestAndCommandName(), MaxTotalRunTimeInSeconds);
				CommandFailedDueToError(ErrorMessageText);
				return true;
			}
		}

		if (IsDelayTimerRunning())
		{
			return false;
		}

		// pre-increment iteration so that its 1 based instead of 0 based for readability
		CurrentIteration++;

		if (!CanRetry())
		{
			// Must log here so that Gauntlet can also pick up the error for its report. Otherwise, it will only show up in the Automation log.
			FString ErrorMessageText = FString::Printf(TEXT("%s Latent command with retries and delays has failed after %d retries"), *GetTestAndCommandName(), MaxRetries);
			CommandFailedDueToError(ErrorMessageText);
			return true;
		}

		ResetDelayTimer();

		// auto-logging for limited retries and unlimited retries
		if (bHasUnlimitedRetries)
		{
			UE_LOGF(LogLatentCommands, Log, "%ls Executing Attempt %d.", *GetTestAndCommandName(), CurrentIteration);
		}
		else
		{
			UE_LOGF(LogLatentCommands, Log, "%ls Executing Attempt %d of %d.", *GetTestAndCommandName(), CurrentIteration, MaxRetries);
		}

		if (Execute())
		{
			// completion log message, logs total time taken.
			UE_LOGF(LogLatentCommands, Log, "%ls Completed Successfully, total run time: %f seconds.", *GetTestAndCommandName(), GetCurrentRunTime());

			return true;
		}

		return false;
	}

	// Pure virtual method that must be overridden.
	// This is the actual command logic.
	virtual bool Execute() = 0;

private:
	// To keep track of which iteration we are on. 1 based
	// To allow for more readable logs. Attempt 1 of N instead
	// of 0 of N.
	int32 CurrentIteration = 0;

protected:
	// default constructor
	IAutomationLatentCommandWithRetriesAndDelays() {}

	// parameterized constructor
	IAutomationLatentCommandWithRetriesAndDelays(const FString InCommandClassName, const int32 InMaxRetries, const double InWaitTimeBetweenRuns)
		:CommandClassName(InCommandClassName)
		, MaxRetries(InMaxRetries < 0 ? 0 : InMaxRetries)
		, bHasUnlimitedRetries(InMaxRetries == 0)
		, DelayTimeInSeconds(InWaitTimeBetweenRuns)
	{
		// set first run start time so that we don't start off in a delay. Its fine if its negative.
		DelayStartTime = FPlatformTime::Seconds() - InWaitTimeBetweenRuns;

		// warn if command has unlimited retries MaxTotalRunTime
		if (bHasUnlimitedRetries)
		{
			// Must log here so that Gauntlet can also pick up the error for its report. Otherwise, it will only show up in the Automation log.
			UE_LOGF(LogLatentCommands, Warning,
				"%ls has been set to Unlimited retries. Will be using MaxTotalRunTime to prevent " "running forever. Default time is set at 300 seconds. If this is not enough time, " "make sure to override this in your Execute() loop.", *GetTestAndCommandName());
		}
	}

	// Resets the Delay Timer by setting start time to now (In game time).
	void ResetDelayTimer()
	{
		DelayStartTime = FPlatformTime::Seconds();
	}

	// Determines if timer is running, pausing execution in a non-blocking way.
	bool IsDelayTimerRunning() const
	{
		// time elapsed < Delay time
		return (FPlatformTime::Seconds() - DelayStartTime) < DelayTimeInSeconds;
	}

	// Returns if we have exceeded max allowed total run time for this latent command
	bool HasExceededMaxTotalRunTime()
	{
		return MaxTotalRunTimeInSeconds - GetCurrentRunTime() < 0;
	}

	// Determines if we should execute the command. If MaxRetries is set to 0, 
	// indicates unlimited retries.
	bool CanRetry() const
	{
		return (CurrentIteration <= MaxRetries) || MaxRetries == 0;
	}

	// Returns the command name 
	FString GetTestAndCommandName() const
	{
		// Otherwise use the provided string
		return FString::Printf(TEXT("Test: %s - Command: %s - "), *GetCurrentTest()->GetTestFullName(), *CommandClassName);
	}

	FAutomationTestBase* GetCurrentTest() const
	{
		return FAutomationTestFramework::Get().GetCurrentTest();
	}

	void OverrideMaxTotalRunTimeInSeconds(double OverrideValue)
	{
		MaxTotalRunTimeInSeconds = OverrideValue;
	}

	// Command Name stored for easy logging
	const FString CommandClassName = "UnknownCommand";

	// Times to retry command before reporting command failure.
	// Defaults to 10, but can be overridden.
	const int32 MaxRetries = 10;

	// Indicates that this latent command has unlimited retries
	// Used to implement the MaxTotalRunTimeInSeconds logic
	const bool bHasUnlimitedRetries = false;

	// Time in between UpdateDelayed calls. 
	// Default is 1 second but can be overridden.
	const double DelayTimeInSeconds = 1.0;

	// Time that the Delay Timer started.
	double DelayStartTime = 0.0;	

private:
	// Max total run time for command 
	// Default is 5 minutes, but can be overridden
	double MaxTotalRunTimeInSeconds = 300.0;
	
};
