// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Internationalization/Text.h"
#include "Math/Axis.h"
#include "Math/Color.h"
#include "Math/MathFwd.h"

namespace AxisDisplayInfo
{
	/**
	 * Gets the axis display coordinate system to use.
	 *
	 * The return value may be driven by the config system and therefore
	 * it should not be cached until the editor has finished loading.
	 *
	 * @return The display coordinate system, either EAxisList::XYZ or EAxisList::LeftUpForward
	 */
	CORE_API EAxisList::Type GetAxisDisplayCoordinateSystem();

	/**
	 * Gets the display name to use for the given axis.
	 * 
	 * The return value may be driven by the config system and therefore
	 * it should not be cached until the editor has finished loading.
	 *
	 * @param Axis The axis whose display name you are requesting
	 * @return The display name corresponding to the given axis
	 */
	CORE_API FText GetAxisDisplayName(EAxisList::Type Axis);

	/**
	 * Gets the short display name to use for the given axis.
	 * 
	 * The return value may be driven by the config system and therefore
	 * it should not be cached until the editor has finished loading.
	 *
	 * @param Axis The axis whose short display name you are requesting
	 * @return The short display name corresponding to the given axis
	 */
	CORE_API FText GetAxisDisplayNameShort(EAxisList::Type Axis);

	CORE_API FText GetAxisToolTip(EAxisList::Type Axis);

	/**
	 * Gets the color to use for the given axis.
	 * 
	 * The return value may be driven by the config system and therefore
	 * it should not be cached until the editor has finished loading.
	 * 
	 * @param Axis The axis whose color you are requesting
	 * @return The color corresponding to the given axis
	 */
	CORE_API FLinearColor GetAxisColor(EAxisList::Type Axis);

	/**
	 * Gets whether or not the engine uses forward/right/up nomenclature for display names.
	 * 
	 * The return value may be driven by the config system and therefore
	 * it should not be cached until the editor has finished loading.
	 */
	CORE_API bool UseForwardRightUpDisplayNames();
	
	CORE_API FText GetRotationAxisName(EAxisList::Type Axis);
	CORE_API FText GetRotationAxisNameShort(EAxisList::Type Axis);
	CORE_API FText GetRotationAxisToolTip(EAxisList::Type Axis);
	

	/**
	 * Default swizzle for displaying axis in transform displays
	 * Swizzle maps from the default X, Y, Z, W positions to new position
	 * { 1, 2, 0, 3 } would map position 0 to fetch Y, position 1 to fetch Z, etc
	 */
	CORE_API FIntVector4 GetTransformAxisSwizzle();
};

class IAxisDisplayInfo : public IModularFeature
{
public:
	virtual ~IAxisDisplayInfo() = default;

	static FName GetModularFeatureName()
	{
		static const FName Name = "AxisDisplayInfo";
		return Name;
	}

	/**
	 * Gets the axis display coordinate system to use.
	 *
	 * The return value may be driven by the config system and therefore
	 * it should not be cached until the editor has finished loading.
	 *
	 * @return The display coordinate system, either EAxisList::XYZ or EAxisList::LeftUpForward
	 */
	virtual EAxisList::Type GetAxisDisplayCoordinateSystem() const = 0;

	/**
	 * Gets the tool tip text used for axis components of widgets describing translations and scales
	 */
	virtual FText GetAxisToolTip(EAxisList::Type Axis) const = 0;

	/**
	 * Gets the display name to use for the given axis.
	 *
	 * @param Axis The axis whose display name you are requesting
	 * @return The display name corresponding to the given axis
	 */
	virtual FText GetAxisDisplayName(EAxisList::Type Axis) = 0;

	/**
	 * Gets the short display name to use for the given axis.
	 *
	 * @param Axis The axis whose short display name you are requesting
	 * @return The short display name corresponding to the given axis
	 */
	virtual FText GetAxisDisplayNameShort(EAxisList::Type Axis) = 0;

	/**
	 * Gets the color to use for the given axis.
	 *
	 * @param Axis The axis whose color you are requesting
	 * @return The color corresponding to the given axis
	 */
	virtual FLinearColor GetAxisColor(EAxisList::Type Axis) = 0;

	/**
	 * Gets the tool tip text used for axis components of widgets describing rotations
	 */
	virtual FText GetRotationAxisToolTip(EAxisList::Type Axis) const = 0;
	/**
	 * Gets the display name use for axis components of widgets describing rotations
	 * Typically this would be used as a label or title
	 */
	virtual FText GetRotationAxisName(EAxisList::Type Axis) = 0;
	/**
	 * Gets a short display name use for axis components of widgets describing rotations
	 * Typically this would be used as a label or title where little room is available
	 */
	virtual FText GetRotationAxisNameShort(EAxisList::Type Axis) = 0;

	/**
	 * Defines the swizzle for slate widgets displaying axis components
	 * such as SVectorInputBox and SRotatorInputBox
	 * Generally, these widgets will be ordered X, Y and Z
	 */
	virtual FIntVector4 DefaultAxisComponentDisplaySwizzle() const = 0;

	/**
	 * Gets whether or not the engine uses forward/right/up nomenclature for display names.
	 */
	virtual bool UseForwardRightUpDisplayNames() = 0;
};
