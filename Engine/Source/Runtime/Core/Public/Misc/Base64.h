// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/Timespan.h"

/** Mode to use for base64 encoding and decoding */
enum class EBase64Mode : uint8
{
	/** Use the standard set of character mappings. (Table 1 from RFC 4648, known as "base64") */
	Standard,
	/** Use the "URL and Filename safe" set of character mappings. (Table 2 from RFC 4648, known as "base64url") */
	UrlSafe
};

/**
 * Class for encoding/decoding Base64 data (RFC 4648)
 */
struct FBase64
{
	/**
	 * Encodes a FString into a Base64 string
	 *
	 * @param Source The string data to convert
	 * @param Mode The mode to use for encoding. Default is EBase64Mode::Standard
	 *
	 * @return A string that encodes the binary data in a way that can be safely transmitted via various Internet protocols
	 */
	static CORE_API FString Encode(const FString& Source, EBase64Mode Mode = EBase64Mode::Standard);

	/**
	 * Encodes a binary uint8 array into a Base64 string
	 *
	 * @param Source The binary data to convert
	 * @param Mode The mode to use for encoding. Default is EBase64Mode::Standard
	 *
	 * @return A string that encodes the binary data in a way that can be safely transmitted via various Internet protocols
	 */
	static CORE_API FString Encode(const TArray<uint8>& Source, EBase64Mode Mode = EBase64Mode::Standard);

	/**
	 * Encodes the source into a Base64 string
	 *
	 * @param Source The binary data to encode
	 * @param Length Length of the binary data to be encoded
	 * @param Mode The mode to use for encoding. Default is EBase64Mode::Standard
	 *
	 * @return Base64 encoded string containing the binary data.
	 */
	static CORE_API FString Encode(const uint8* Source, uint32 Length, EBase64Mode Mode = EBase64Mode::Standard);

	/**
	 * Encodes the source into a Base64 string, storing it in a preallocated buffer.
	 *
	 * @param Source The binary data to encode
	 * @param Length Length of the binary data to be encoded
	 * @param Dest Buffer to receive the encoded data. Must be large enough to contain the entire output data (see GetEncodedDataSize()). Can point to the same buffer as Source
	 * @param Mode The mode to use for encoding. Default is EBase64Mode::Standard
	 *
	 * @return The length of the encoded data
	 */
	template<typename CharType>
	CORE_API static uint32 Encode(const uint8* Source, uint32 Length, CharType* Dest, EBase64Mode Mode = EBase64Mode::Standard);

	/**
	* Get the encoded data size for the given number of bytes.
	*
	* @param NumBytes The number of bytes of input
	*
	* @return The number of characters in the encoded data.
	*/
	static inline constexpr uint32 GetEncodedDataSize(uint32 NumBytes)
	{
		return ((NumBytes + 2) / 3) * 4;
	}

	/**
	 * Decodes a Base64 string into a FString
	 *
	 * @param Source The Base64 encoded string
	 * @param OutDest Receives the decoded string data
	 * @param Mode The mode to use for decoding. Default is EBase64Mode::Standard
	 *
	 * @return true if the buffer was decoded, false if it was invalid.
	 */
	static CORE_API bool Decode(const FString& Source, FString& OutDest, EBase64Mode Mode = EBase64Mode::Standard);

	/**
	 * Decodes a Base64 string into an array of bytes
	 *
	 * @param Source The Base64 encoded string
	 * @param Dest Array to receive the decoded data
	 * @param Mode The mode to use for decoding. Default is EBase64Mode::Standard
	 *
	 * @return true if the buffer was decoded, false if it was invalid.
	 */
	static CORE_API bool Decode(const FString& Source, TArray<uint8>& Dest, EBase64Mode Mode = EBase64Mode::Standard);

	/**
	 * Decodes a Base64 string into a preallocated buffer
	 *
	 * @param Source The Base64 encoded string
	 * @param Length Length of the Base64 encoded string
	 * @param Dest Buffer to receive the decoded data.  Must be large enough to contain the entire output data (see GetDecodedDataSize()). Can point to the same buffer as Source
	 * @param Mode The mode to use for decoding. Default is EBase64Mode::Standard
	 *
	 * @return true if the buffer was decoded, false if it was invalid.
	 */
	template<typename CharType>
	CORE_API static bool Decode(const CharType* Source, uint32 Length, uint8* Dest, EBase64Mode Mode = EBase64Mode::Standard);

	/**
	* Determine the decoded data size for the incoming base64 encoded string
	*
	* @param Source The Base64 encoded string
	*
	* @return The size in bytes of the decoded data
	*/
	static CORE_API uint32 GetDecodedDataSize(const FString& Source);

	/**
	* Determine the decoded data size for the incoming base64 encoded string
	*
	* @param Source The Base64 encoded string
	* @param Length Length of the Base64 encoded string
	*
	* @return The size in bytes of the decoded data
	*/
	template<typename CharType>
	CORE_API static uint32 GetDecodedDataSize(const CharType* Source, uint32 Length);

	/**
	* Get the maximum decoded data size for the given number of input characters.
	*
	* @param Length The number of input characters.
	*
	* @return The maximum number of bytes that can be decoded from this input stream. The actual number of characters decoded may be less if the data contains padding characters. Call GetDecodedDataSize() with the input data to find the exact length.
	*/
	static inline constexpr uint32 GetMaxDecodedDataSize(uint32 NumChars)
	{
		return ((NumChars * 3) + 3) / 4;
	}
};
