// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "Logging/LogVerbosity.h"
#include "Misc/OutputDevice.h"
#include "Misc/OutputDeviceRedirector.h"

/** Buffered output device. */
class FBufferedOutputDevice : public FOutputDevice
{
protected:
	TArray<FBufferedLine>		BufferedLines;
	FCriticalSection			SynchronizationObject;
	ELogVerbosity::Type			FilterLevel = ELogVerbosity::All;

public:
	void	SetVerbosity(ELogVerbosity::Type Verbosity) { FilterLevel = Verbosity; }
	CORE_API void	Serialize(const TCHAR* InData, ELogVerbosity::Type Verbosity, const FName& Category) override;
	CORE_API void	GetContents(TArray<FBufferedLine>& DestBuffer);

	/** Pushes buffered lines into the specified output device. */
	CORE_API void	RedirectTo(FOutputDevice& Ar);

	CORE_API FBufferedOutputDevice();
	CORE_API virtual ~FBufferedOutputDevice();
};
