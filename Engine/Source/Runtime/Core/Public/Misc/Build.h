// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*--------------------------------------------------------------------------------
	Build configuration coming from UBT, do not modify
--------------------------------------------------------------------------------*/

// Set any configuration not defined by UBT to zero
#ifndef UE_BUILD_DEBUG
	#define UE_BUILD_DEBUG				0
#endif
#ifndef UE_BUILD_DEVELOPMENT
	#define UE_BUILD_DEVELOPMENT		0
#endif
#ifndef UE_BUILD_TEST
	#define UE_BUILD_TEST				0
#endif
#ifndef UE_BUILD_SHIPPING
	#define UE_BUILD_SHIPPING			0
#endif
#ifndef UE_GAME
	#define UE_GAME						0
#endif
#ifndef UE_CLIENT
	#define UE_CLIENT					0
#endif
#ifndef UE_EDITOR
	#define UE_EDITOR					0
#endif
#ifndef UE_BUILD_SHIPPING_WITH_EDITOR
	#define UE_BUILD_SHIPPING_WITH_EDITOR 0
#endif
#ifndef UE_BUILD_DOCS
	#define UE_BUILD_DOCS				0
#endif

/** 
 *   Whether compiling for dedicated server or not.
 */
#ifndef UE_SERVER
	#define UE_SERVER					0
#endif

// Ensure that we have one, and only one build config coming from UBT
#if UE_BUILD_DEBUG + UE_BUILD_DEVELOPMENT + UE_BUILD_TEST + UE_BUILD_SHIPPING != 1
	#error Exactly one of [UE_BUILD_DEBUG UE_BUILD_DEVELOPMENT UE_BUILD_TEST UE_BUILD_SHIPPING] should be defined to be 1
#endif

/*--------------------------------------------------------------------------------
	Legacy defined we want to make sure don't compile if they came in a merge.
--------------------------------------------------------------------------------*/

#define FINAL_RELEASE_DEBUGCONSOLE	(#)
#define FINAL_RELEASE				(#)
#define SHIPPING_PC_GAME			(#)
#define UE_BUILD_FINAL_RELEASE (#)

/*----------------------------------------------------------------------------
	Mandatory bridge options coming from UBT, do not modify directly!
----------------------------------------------------------------------------*/

/**
 * Whether we are compiling with the editor; must be defined by UBT
 */
#ifndef WITH_EDITOR
	#define WITH_EDITOR	0 // for auto-complete
	#error UBT should always define WITH_EDITOR to be 0 or 1
#endif

/**
 * Whether we are compiling with the engine; must be defined by UBT
 */
#ifndef WITH_ENGINE
	#define WITH_ENGINE	0 // for auto-complete
	#error UBT should always define WITH_ENGINE to be 0 or 1
#endif

/**
 *	Whether we are compiling with developer tools; must be defined by UBT
 */
#ifndef WITH_UNREAL_DEVELOPER_TOOLS
	#define WITH_UNREAL_DEVELOPER_TOOLS		0	// for auto-complete
	#error UBT should always define WITH_UNREAL_DEVELOPER_TOOLS to be 0 or 1
#endif

 /**
  *	Whether we are compiling with developer tools that may use other platforms or external connected devices, etc
  */
#ifndef WITH_UNREAL_TARGET_DEVELOPER_TOOLS
	#define WITH_UNREAL_TARGET_DEVELOPER_TOOLS		WITH_UNREAL_DEVELOPER_TOOLS // a subset of WITH_UNREAL_DEVELOPER_TOOLS, but can be disabled separately
#endif

/**
 *	Whether we are compiling with plugin support; must be defined by UBT
 */
#ifndef WITH_PLUGIN_SUPPORT
	#define WITH_PLUGIN_SUPPORT		0	// for auto-complete
	#error UBT should always define WITH_PLUGIN_SUPPORT to be 0 or 1
#endif

/**
 * Whether we are compiling with Slate accessibility and automation support
 */
#ifndef WITH_ACCESSIBILITY
	#define WITH_ACCESSIBILITY		1
#endif

 /** Enable perf counters */
#ifndef WITH_PERFCOUNTERS
	#define WITH_PERFCOUNTERS		0
#endif

/** Enable perf counters on dedicated servers */
#define USE_SERVER_PERF_COUNTERS ((UE_SERVER || UE_EDITOR) && WITH_PERFCOUNTERS)

/** 
 * Whether we are compiling a PGO instrumented build.
 */
#ifndef ENABLE_PGO_PROFILE
	#define ENABLE_PGO_PROFILE 0
#endif

/** Whether we are compiling with automation worker functionality.  Note that automation worker defaults to enabled in
    UE_BUILD_TEST configuration, so that it can be used for performance testing on devices */
#ifndef WITH_AUTOMATION_WORKER
	#define WITH_AUTOMATION_WORKER !UE_BUILD_SHIPPING
#endif

/**
* Whether we want a monolithic build (no DLLs); must be defined by UBT
*/
#ifndef IS_MONOLITHIC
	#define IS_MONOLITHIC 0 // for auto-complete
	#error UBT should always define IS_MONOLITHIC to be 0 or 1
#endif

/**
* Whether we want a program (shadercompilerworker, fileserver) or a game; must be defined by UBT
*/
#ifndef IS_PROGRAM
	#define IS_PROGRAM 0 // for autocomplete
	#error UBT should always define IS_PROGRAM to be 0 or 1
#endif

/**
* Whether we support hot-reload. Currently requires a non-monolithic build and non-shipping configuration.
*/
#ifndef WITH_HOT_RELOAD
	#define WITH_HOT_RELOAD (!IS_MONOLITHIC && !UE_BUILD_SHIPPING && !UE_BUILD_TEST && !UE_GAME && !UE_SERVER)
#endif

/**
* Make sure that live coding define is available.  Normally this is supplied by UBT
*/
#ifndef WITH_LIVE_CODING
	#define WITH_LIVE_CODING 0
#endif

/**
* Whether we support any type of live reloading
*/
#define WITH_RELOAD (WITH_HOT_RELOAD || WITH_LIVE_CODING)

/**
* Whether we include support for text archive formats. Disabling support allows de-virtualizing archive calls
* and eliminating string constants for field names.
*/
#ifndef WITH_TEXT_ARCHIVE_SUPPORT
	#define WITH_TEXT_ARCHIVE_SUPPORT WITH_EDITORONLY_DATA
#endif

// Statestream is WIP and is the system that is going to enable full decoupling of game and render updates
// Set in .target.cs to enable
#ifndef WITH_STATE_STREAM
	#define WITH_STATE_STREAM 0
#endif


/*----------------------------------------------------------------------------
	Optional bridge options coming from UBT, do not modify directly!
	If UBT doesn't set the value, it is assumed to be 0, and we set that here.
----------------------------------------------------------------------------*/

/**
 * Checks to see if pure virtual has actually been implemented, this is normally run as a CIS process and is set (indirectly) by UBT
 *
 * @see Core.h
 * @see ObjectMacros.h
 **/
#ifndef CHECK_PUREVIRTUALS
	#define CHECK_PUREVIRTUALS 0
#endif

/** Whether to use the null RHI. */
#ifndef USE_NULL_RHI
	#define USE_NULL_RHI 0
#endif

/** If not specified, disable logging in shipping */
#ifndef USE_LOGGING_IN_SHIPPING
	#define USE_LOGGING_IN_SHIPPING 0
#endif

#ifndef USE_CHECKS_IN_SHIPPING
	#define USE_CHECKS_IN_SHIPPING 0
#endif

/** If not defined follow the CHECK behavior since previously ensures were compiled in with checks */
#ifndef USE_ENSURES_IN_SHIPPING
	#define USE_ENSURES_IN_SHIPPING USE_CHECKS_IN_SHIPPING
#endif

#ifndef ALLOW_CONSOLE_IN_SHIPPING
	#define ALLOW_CONSOLE_IN_SHIPPING 0
#endif

/** Compile flag to force stats to be compiled */
#ifndef FORCE_USE_STATS
	#define FORCE_USE_STATS 0
#endif

/** Set to true to force an ansi allocator instead of redirecting to FMemory. */
#ifndef FORCE_ANSI_ALLOCATOR
	#define FORCE_ANSI_ALLOCATOR 0
#endif

/*--------------------------------------------------------------------------------
	Basic options that by default depend on the build configuration and platform

	DO_GUARD_SLOW									If true, then checkSlow, checkfSlow and verifySlow are compiled into the executable.
	DO_CHECK										If true, then checkCode, checkf, verify, check, checkNoEntry, checkNoReentry, checkNoRecursion, verifyf, checkf are compiled into the executables
	DO_ENSURE										If true, then ensure, ensureAlways, ensureMsgf and ensureAlwaysMsgf are compiled into the executables
	STATS											If true, then the stats system is compiled into the executable.
	ALLOW_DEBUG_FILES								If true, then debug files like screen shots and profiles can be saved from the executable.
	NO_LOGGING										If true, then no logs or text output will be produced
	ENABLE_STATNAMEDEVENTS							If true, then enable support for named events from the stat macros without the stat system overhead. This will attempt to disable regular stats system and use named events instead.

--------------------------------------------------------------------------------*/

#if UE_BUILD_DEBUG
	#ifndef DO_GUARD_SLOW
		#define DO_GUARD_SLOW									1
	#endif
	#ifndef DO_CHECK
		#define DO_CHECK										1
	#endif
	#ifndef DO_ENSURE
		#define DO_ENSURE										1
	#endif
	#ifndef ENABLE_STATNAMEDEVENTS
		#define ENABLE_STATNAMEDEVENTS							0
	#endif
	#ifndef ENABLE_STATNAMEDEVENTS_UOBJECT
		#define ENABLE_STATNAMEDEVENTS_UOBJECT 					0
	#endif
	#ifndef STATS
		#define STATS											((WITH_UNREAL_DEVELOPER_TOOLS || !WITH_EDITORONLY_DATA || USE_STATS_WITHOUT_ENGINE || FORCE_USE_STATS) && !ENABLE_STATNAMEDEVENTS)
	#endif
	#ifndef ALLOW_DEBUG_FILES
		#define ALLOW_DEBUG_FILES								1
	#endif
	#ifndef ALLOW_CONSOLE
		#define ALLOW_CONSOLE									1
	#endif
	#ifndef NO_LOGGING
		#define NO_LOGGING										0
	#endif
#elif UE_BUILD_DEVELOPMENT
	#ifndef DO_GUARD_SLOW
		#define DO_GUARD_SLOW									0
	#endif
	#ifndef DO_CHECK
		#define DO_CHECK										1
	#endif
	#ifndef DO_ENSURE
		#define DO_ENSURE										1
	#endif
	#ifndef ENABLE_STATNAMEDEVENTS
		#define ENABLE_STATNAMEDEVENTS							0
	#endif
	#ifndef ENABLE_STATNAMEDEVENTS_UOBJECT
		#define ENABLE_STATNAMEDEVENTS_UOBJECT 					0
	#endif
	#ifndef STATS
		#define STATS											((WITH_UNREAL_DEVELOPER_TOOLS || !WITH_EDITORONLY_DATA || USE_STATS_WITHOUT_ENGINE || FORCE_USE_STATS) && !ENABLE_STATNAMEDEVENTS)
	#endif
	#ifndef ALLOW_DEBUG_FILES
		#define ALLOW_DEBUG_FILES								1
	#endif
	#ifndef ALLOW_CONSOLE
		#define ALLOW_CONSOLE									1
	#endif
	#ifndef NO_LOGGING
		#define NO_LOGGING										0
	#endif
#elif UE_BUILD_TEST
	#ifndef DO_GUARD_SLOW
		#define DO_GUARD_SLOW									0
	#endif
	#ifndef DO_CHECK
		#define DO_CHECK										USE_CHECKS_IN_SHIPPING
	#endif
	#ifndef DO_ENSURE
		#define DO_ENSURE										USE_ENSURES_IN_SHIPPING
	#endif
	#ifndef ENABLE_STATNAMEDEVENTS
		#define ENABLE_STATNAMEDEVENTS							(!FORCE_USE_STATS && !USE_STATS_WITHOUT_ENGINE)
	#endif
	#ifndef ENABLE_STATNAMEDEVENTS_UOBJECT
		#define ENABLE_STATNAMEDEVENTS_UOBJECT 					(ENABLE_STATNAMEDEVENTS)
	#endif
	#ifndef STATS
		#define STATS											(FORCE_USE_STATS && !ENABLE_STATNAMEDEVENTS)
	#endif
	#ifndef ALLOW_DEBUG_FILES
		#define ALLOW_DEBUG_FILES								1
	#endif
	#ifndef ALLOW_CONSOLE
		#define ALLOW_CONSOLE									1
	#endif
	#ifndef NO_LOGGING
		#define NO_LOGGING										!USE_LOGGING_IN_SHIPPING
	#endif
#elif UE_BUILD_SHIPPING
	#ifndef DO_GUARD_SLOW
		#define DO_GUARD_SLOW								0
	#endif
	#ifndef DO_CHECK
		#define DO_CHECK									USE_CHECKS_IN_SHIPPING
	#endif
	#ifndef DO_ENSURE
		#define DO_ENSURE									USE_ENSURES_IN_SHIPPING
	#endif
	#ifndef ENABLE_STATNAMEDEVENTS
		#define ENABLE_STATNAMEDEVENTS						0
	#endif
	#ifndef ENABLE_STATNAMEDEVENTS_UOBJECT
		#define ENABLE_STATNAMEDEVENTS_UOBJECT 				0
	#endif
	#ifndef STATS
		#define STATS										(FORCE_USE_STATS && !ENABLE_STATNAMEDEVENTS)
	#endif
	#ifndef ALLOW_DEBUG_FILES
		#define ALLOW_DEBUG_FILES							WITH_EDITOR
	#endif
	#ifndef ALLOW_CONSOLE
		#define ALLOW_CONSOLE								ALLOW_CONSOLE_IN_SHIPPING
	#endif
	#ifndef NO_LOGGING
		#define NO_LOGGING									!USE_LOGGING_IN_SHIPPING
	#endif
#else
	#error Exactly one of [UE_BUILD_DEBUG UE_BUILD_DEVELOPMENT UE_BUILD_TEST UE_BUILD_SHIPPING] should be defined to be 1
#endif


/**
 * This is a global setting which will turn on logging / checks for things which are
 * considered especially bad for consoles.  Some of the checks are probably useful for PCs also.
 *
 * Throughout the code base there are specific things which dramatically affect performance and/or
 * are good indicators that something is wrong with the content.  These have PERF_ISSUE_FINDER in the
 * comment near the define to turn the individual checks on. 
 *
 * e.g. #if defined(PERF_LOG_DYNAMIC_LOAD_OBJECT) || LOOKING_FOR_PERF_ISSUES
 *
 * If one only cares about DLO, then one can enable the PERF_LOG_DYNAMIC_LOAD_OBJECT define.  Or one can
 * globally turn on all PERF_ISSUE_FINDERS :-)
 *
 **/
#ifndef LOOKING_FOR_PERF_ISSUES
	#define LOOKING_FOR_PERF_ISSUES (0 && !(UE_BUILD_SHIPPING))
#endif

/** Enable the use of the network profiler as long as we are not a Shipping or Test build */
#ifndef USE_NETWORK_PROFILER
#define USE_NETWORK_PROFILER !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif

/** Enable validation of the Uber Graph's persistent frame's layout, this is useful to detect uber graph frame related corruption */
#ifndef VALIDATE_USER_GRAPH_PERSISTENT_FRAME
	#define VALIDATE_UBER_GRAPH_PERSISTENT_FRAME (!(UE_BUILD_SHIPPING || UE_BUILD_TEST))
#endif

/** Enable fast calls for event thunks into an event graph that have no parameters  */
#ifndef UE_BLUEPRINT_EVENTGRAPH_FASTCALLS
	#define UE_BLUEPRINT_EVENTGRAPH_FASTCALLS 1
#endif

/** Enables code required for handling recursive dependencies during blueprint serialization */
#ifndef USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	#define USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING 1
#endif

/** Enable validation of deferred dependencies loaded during blueprint serialization */
#ifndef USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	#define USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS (USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING && 0)
#endif

// 0 (default), set this to 1 to get draw events with "TOGGLEDRAWEVENTS" "r.ShowMaterialDrawEvents" and the "ProfileGPU" command working in test
#ifndef ALLOW_PROFILEGPU_IN_TEST
	#define ALLOW_PROFILEGPU_IN_TEST 0
#endif

#ifndef ALLOW_PROFILEGPU_IN_SHIPPING
	#define ALLOW_PROFILEGPU_IN_SHIPPING 0
#endif

// draw events with "TOGGLEDRAWEVENTS" "r.ShowMaterialDrawEvents" (for ProfileGPU, Pix, Razor, RenderDoc, ...) and the "ProfileGPU" command are normally compiled out for TEST and SHIPPING
#define WITH_PROFILEGPU (!(UE_BUILD_SHIPPING || UE_BUILD_TEST) || (UE_BUILD_TEST && ALLOW_PROFILEGPU_IN_TEST) || (UE_BUILD_SHIPPING && ALLOW_PROFILEGPU_IN_SHIPPING))

#ifndef ALLOW_DUMPGPU_IN_TEST
	#define ALLOW_DUMPGPU_IN_TEST 1
#endif

#ifndef ALLOW_DUMPGPU_IN_SHIPPING
	#define ALLOW_DUMPGPU_IN_SHIPPING 0
#endif

// DumpGPU command
#define WITH_DUMPGPU (!(UE_BUILD_SHIPPING || UE_BUILD_TEST) || (UE_BUILD_TEST && ALLOW_DUMPGPU_IN_TEST) || (UE_BUILD_SHIPPING && ALLOW_DUMPGPU_IN_SHIPPING))

#ifndef ALLOW_GPUDEBUGCRASH_IN_TEST
#define ALLOW_GPUDEBUGCRASH_IN_TEST 1
#endif

#ifndef ALLOW_GPUDEBUGCRASH_IN_SHIPPING
#define ALLOW_GPUDEBUGCRASH_IN_SHIPPING 0
#endif

// GPUDebugCrash
#define WITH_GPUDEBUGCRASH (!(UE_BUILD_SHIPPING || UE_BUILD_TEST) || (UE_BUILD_TEST && ALLOW_GPUDEBUGCRASH_IN_TEST) || (UE_BUILD_SHIPPING && ALLOW_GPUDEBUGCRASH_IN_SHIPPING))

#ifndef ALLOW_CHEAT_CVARS_IN_TEST
	#define ALLOW_CHEAT_CVARS_IN_TEST 1
#endif

#ifndef DISABLE_CHEAT_CVARS
	#define DISABLE_CHEAT_CVARS (UE_BUILD_SHIPPING || (UE_BUILD_TEST && !ALLOW_CHEAT_CVARS_IN_TEST))
#endif

// Controls the creation of a thread for detecting hangs (FThreadHeartBeat). This is subject to other criteria, USE_HANG_DETECTION
#ifndef ALLOW_HANG_DETECTION
	#define ALLOW_HANG_DETECTION 1
#endif
#define USE_HANG_DETECTION (ALLOW_HANG_DETECTION && !WITH_EDITORONLY_DATA && !IS_PROGRAM && !UE_BUILD_DEBUG && !ENABLE_PGO_PROFILE && !USING_THREAD_SANITISER && !USING_INSTRUMENTATION)

// Controls the creation of a thread for detecting hitches (FGameThreadHitchHeartBeat). This is subject to other criteria, USE_HITCH_DETECTION
#ifndef ALLOW_HITCH_DETECTION
	#define ALLOW_HITCH_DETECTION 0
#endif

// Adjust a few things with the slack policy and MallocBinned2 to minimize memory usage (at some performance cost)
#ifndef AGGRESSIVE_MEMORY_SAVING
	#define AGGRESSIVE_MEMORY_SAVING 0
#endif

// Controls if UObjects are initialized as soon as they are available or only after the module is "loaded". This only applies to monolithic builds; if there are DLLs, this is how it works anyway and this should not be turned on
#ifndef USE_PER_MODULE_UOBJECT_BOOTSTRAP
	#define USE_PER_MODULE_UOBJECT_BOOTSTRAP 0
#endif

#ifndef USE_HITCH_DETECTION
	#define USE_HITCH_DETECTION (ALLOW_HITCH_DETECTION && !WITH_EDITORONLY_DATA && !IS_PROGRAM && !UE_BUILD_DEBUG && !USING_THREAD_SANITISER && !USING_INSTRUMENTATION)
#endif

// Controls whether shipping builds create backups of the most recent log file.
// All other configurations always create backups.
#ifndef PRESERVE_LOG_BACKUPS_IN_SHIPPING
	#define PRESERVE_LOG_BACKUPS_IN_SHIPPING 1
#endif

#ifndef ENABLE_RHI_VALIDATION
	#define ENABLE_RHI_VALIDATION (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)
#endif

// Controls whether FPlatformMisc::GetDeviceId() is available to be called.
// When set to 1, calls to this API will be hardcoded to return an empty string
// to avoid running afoul of calling device APIs that platform owners may restrict
// access to without waivers or special steps. Code that uses GetDeviceId() should
// expect to receive empty strings in these cases and response appropriately with
// fallback logic.
#ifndef GET_DEVICE_ID_UNAVAILABLE
	#define GET_DEVICE_ID_UNAVAILABLE 0
#endif

// Controls whether the executable is compiled with cooked editor functionality
#ifndef UE_IS_COOKED_EDITOR
#define UE_IS_COOKED_EDITOR 0
#endif
// Controls whether to enable loading cooked packages from I/O store in editor builds. Defaults to WITH_EDITOR as we can now
// load from IoStore without needing to stage an entire cooked build
#ifndef WITH_IOSTORE_IN_EDITOR
#define WITH_IOSTORE_IN_EDITOR WITH_EDITOR
#endif

// Controls if iostore will be forced on
#ifndef UE_FORCE_USE_IOSTORE
#define UE_FORCE_USE_IOSTORE 0
#endif

// Controls if paks will be forced on unless -NoPaks argument is passed
#ifndef UE_FORCE_USE_PAKS
#define UE_FORCE_USE_PAKS 0
#endif


// Controls whether Iris networking code is compiled in or not; should normally be defined by UBT
#ifndef UE_WITH_IRIS
	#define UE_WITH_IRIS 0
#endif

// Controls whether or not to make a global object to load COnfig.bin as soon as possible
#ifndef PRELOAD_BINARY_CONFIG
	#define PRELOAD_BINARY_CONFIG 1
#endif

#ifndef WITH_COTF
	#define WITH_COTF ((WITH_ENGINE) && !(IS_PROGRAM || UE_BUILD_SHIPPING))
#endif

// Controls if the config system can stores configs for other platforms than the running one
#ifndef ALLOW_OTHER_PLATFORM_CONFIG
#define ALLOW_OTHER_PLATFORM_CONFIG		WITH_UNREAL_DEVELOPER_TOOLS
#endif

// Controls whether or not process will control OS scheduler priority
#ifndef WITH_PROCESS_PRIORITY_CONTROL
	#define WITH_PROCESS_PRIORITY_CONTROL 0
#endif

// A compile time flag used to enable support for disabling actor ticking and calls to user callbacks.
// This functionality is not intended for general use and should be used with care.
#ifndef UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
	#define UE_SUPPORT_FOR_ACTOR_TICK_DISABLE 0
#endif

// Enables the universal placeholders feature, which allows all imported objects to be replaced with 
// placeholder objects. This flag is internal and will dispear in the future without warning.
// It is used to facilitate fast visualization of tagged property .uasset data by satisfying the 
// following needs:
// 
//	1.	Assume no imported packages need to actually be loaded, and can therefore be universally mocked, 
//		hence 'universal placeholders' - see FLinkerLoad::CreateUniversalPlaceholders
//	2.	Provide safe behavior to placeholders which would need their type to 'emerge' from usage. 
//		Currently placeholders are mocked as plain objects, but that cannot accomodate native types with 
//		native invariants - see ObjectPtr.h
//	3.	Relatedly avoids discarding placeholders that don't conform to emergent non-native type 
//		constraints see FLinkerLoad::operator<<(FObjectPtr&)
//	4.	Relatedly avoids hashing of placeholder types that are not hashable - see PropertySet.cpp,
//		PropertyMap.cpp
//  5.	Assumes a read operation, avoiding expensive mutators - see PropertyBagRepository.cpp
//	6.	Allows serialization routines to expect unreachable objects (in the long run all loaded 
//		data should be made reachable) - see JsonStringifyImpl.cpp
//
// All of these usages will be accomodated more elegantly by future work. Do not use this define.
#ifndef UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
	#define UE_INTERNAL_UNIVERSAL_PLACEHOLDERS 0
#endif

// Universal placeholders are incompatible with full editor functionality, preventing use in that
// configuration for safety:
#if UE_EDITOR && UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
	#error UE_INTERNAL_UNIVERSAL_PLACEHOLDERS cannot be used in the full UE_EDITOR
#endif

// Universal placeholders are also incompatible with pure cooked configurations, as universal placeholders
// rely on tagged serialization, IDOs, placeholders and other editor only FLinkerLoad functionality:
#if !WITH_EDITOR && UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
	#error UE_INTERNAL_UNIVERSAL_PLACEHOLDERS cannot be used in cooked configurations
#endif

// This define adds more robust bounds checking to FLinkerLoad:
#if WITH_EDITOR
	#define UE_INTERNAL_LINKER_BOUNDS 1
#else 
	#define UE_INTERNAL_LINKER_BOUNDS 0
#endif
