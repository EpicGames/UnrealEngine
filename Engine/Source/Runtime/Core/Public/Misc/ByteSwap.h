// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"

// These macros are not safe to use unless data is UNSIGNED!
#define BYTESWAP_ORDER16_unsigned(x) ((((x) >> 8) & 0xff) + (((x) << 8) & 0xff00))
#define BYTESWAP_ORDER32_unsigned(x) (((x) >> 24) + (((x) >> 8) & 0xff00) + (((x) << 8) & 0xff0000) + ((x) << 24))

// Encapsulate Byte swapping generic versions for benchmarking purpose (compare the generic vs intrinsic performance: See ByteSwapTest.cpp).
namespace Internal
{
	static UE_FORCEINLINE_HINT uint16 ByteSwapGeneric16(uint16 Value)
	{
		return (BYTESWAP_ORDER16_unsigned(Value));
	}

	static UE_FORCEINLINE_HINT uint32 ByteSwapGeneric32(uint32 Value)
	{
		return (BYTESWAP_ORDER32_unsigned(Value));
	}

	static inline uint64 ByteSwapGeneric64(uint64 Value)
	{
		Value = ((Value << 8) & 0xFF00FF00FF00FF00ULL ) | ((Value >> 8) & 0x00FF00FF00FF00FFULL);
		Value = ((Value << 16) & 0xFFFF0000FFFF0000ULL ) | ((Value >> 16) & 0x0000FFFF0000FFFFULL);
		return (Value << 32) | (Value >> 32);
	}
} // namespace Internal

// Defines the intrinsic version if available (faster). Do not use directly. Use BYTESWAP_ORDERxx() or ByteSwap() functions
#if defined(_MSC_VER)
	#define UE_BYTESWAP_INTRINSIC_PRIVATE_16(Val) _byteswap_ushort(Val);
	#define UE_BYTESWAP_INTRINSIC_PRIVATE_32(Val) _byteswap_ulong(Val);
	#define UE_BYTESWAP_INTRINSIC_PRIVATE_64(Val) _byteswap_uint64(Val);
#elif defined(__clang__)
	#if (__has_builtin(__builtin_bswap16))
		#define UE_BYTESWAP_INTRINSIC_PRIVATE_16(Val) __builtin_bswap16(Val);
	#endif
	#if (__has_builtin(__builtin_bswap32))
		#define UE_BYTESWAP_INTRINSIC_PRIVATE_32(Val) __builtin_bswap32(Val);
	#endif
	#if (__has_builtin(__builtin_bswap64))
		#define UE_BYTESWAP_INTRINSIC_PRIVATE_64(Val) __builtin_bswap64(Val);
	#endif
#endif

inline uint16 BYTESWAP_ORDER16(uint16 Val)
{
#if defined(UE_BYTESWAP_INTRINSIC_PRIVATE_16)
	return UE_BYTESWAP_INTRINSIC_PRIVATE_16(Val);
#else
	return Internal::ByteSwapGeneric16(Val);
#endif
}

UE_FORCEINLINE_HINT int16 BYTESWAP_ORDER16(int16 Val)
{
	return static_cast<int16>(BYTESWAP_ORDER16(static_cast<uint16>(Val)));
}

inline uint32 BYTESWAP_ORDER32(uint32 Val)
{
#if defined(UE_BYTESWAP_INTRINSIC_PRIVATE_32)
	return UE_BYTESWAP_INTRINSIC_PRIVATE_32(Val);
#else
	return Internal::ByteSwapGeneric32(Val);
#endif
}

UE_FORCEINLINE_HINT int32 BYTESWAP_ORDER32(int32 val)
{
	return static_cast<int32>(BYTESWAP_ORDER32(static_cast<uint32>(val)));
}

inline uint64 BYTESWAP_ORDER64(uint64 Value)
{
#if defined(UE_BYTESWAP_INTRINSIC_PRIVATE_64)
	return UE_BYTESWAP_INTRINSIC_PRIVATE_64(Value);
#else
	return Internal::ByteSwapGeneric64(Value);
#endif
}

UE_FORCEINLINE_HINT int64 BYTESWAP_ORDER64(int64 Value)
{
	return static_cast<int64>(BYTESWAP_ORDER64(static_cast<uint64>(Value)));
}

inline float BYTESWAP_ORDERF(float val)
{
	uint32 uval = BYTESWAP_ORDER32(*reinterpret_cast<const uint32*>(&val));
	return *reinterpret_cast<const float*>(&uval);
}

inline double BYTESWAP_ORDERD(double val)
{
	uint64 uval = BYTESWAP_ORDER64(*reinterpret_cast<const uint64*>(&val));
	return *reinterpret_cast<const double*>(&uval);
}

#if PLATFORM_TCHAR_IS_UTF8CHAR

UE_FORCEINLINE_HINT void BYTESWAP_ORDER_TCHARARRAY(TCHAR* str)
{
	static_assert(sizeof(TCHAR) == 1, "Assuming TCHAR is 1 byte wide.");

	// No need to byteswap single bytes.
}

#else

inline void BYTESWAP_ORDER_TCHARARRAY(TCHAR* str)
{
	static_assert(sizeof(TCHAR) == sizeof(uint16), "Assuming TCHAR is 2 bytes wide.");
	for (TCHAR* c = str; *c; ++c)
	{
		*c = BYTESWAP_ORDER16(static_cast<uint16>(*c));
	}
}

#endif

// General byte swapping.
#if PLATFORM_LITTLE_ENDIAN
	#define INTEL_ORDER16(x)   (x)
	#define INTEL_ORDER32(x)   (x)
	#define INTEL_ORDERF(x)    (x)
	#define INTEL_ORDER64(x)   (x)
	#define INTEL_ORDER_TCHARARRAY(x)
	#define NETWORK_ORDER16(x)			BYTESWAP_ORDER16(x)
	#define NETWORK_ORDER32(x)			BYTESWAP_ORDER32(x)
	#define NETWORK_ORDERF(x)			BYTESWAP_ORDERF(x)
	#define NETWORK_ORDER64(x)			BYTESWAP_ORDER64(x)
	#define NETWORK_ORDER_TCHARARRAY(x)	BYTESWAP_ORDER_TCHARARRAY(x)
#else
	#define INTEL_ORDER16(x)			BYTESWAP_ORDER16(x)
	#define INTEL_ORDER32(x)			BYTESWAP_ORDER32(x)
	#define INTEL_ORDERF(x)				BYTESWAP_ORDERF(x)
	#define INTEL_ORDER64(x)			BYTESWAP_ORDER64(x)
	#define INTEL_ORDER_TCHARARRAY(x)	BYTESWAP_ORDER_TCHARARRAY(x)
	#define NETWORK_ORDER16(x)   (x)
	#define NETWORK_ORDER32(x)   (x)
	#define NETWORK_ORDERF(x)    (x)
	#define NETWORK_ORDER64(x)   (x)
	#define NETWORK_ORDER_TCHARARRAY(x)
#endif

// Implementations that better mix with generic template code.
template<typename T> T ByteSwap(T Value); // Left unimplemented to get link error when a specialization is missing.
template<> inline int16 ByteSwap(int16 Value)   { return BYTESWAP_ORDER16(Value); }
template<> inline uint16 ByteSwap(uint16 Value) { return BYTESWAP_ORDER16(Value); }
template<> inline int32 ByteSwap(int32 Value)   { return BYTESWAP_ORDER32(Value); }
template<> inline uint32 ByteSwap(uint32 Value) { return BYTESWAP_ORDER32(Value); }
template<> inline int64 ByteSwap(int64 Value)   { return BYTESWAP_ORDER64(Value); }
template<> inline uint64 ByteSwap(uint64 Value) { return BYTESWAP_ORDER64(Value); }
template<> inline float ByteSwap(float Value)   { return BYTESWAP_ORDERF(Value); }
template<> inline double ByteSwap(double Value) { return BYTESWAP_ORDERD(Value); }
template<> inline char16_t ByteSwap(char16_t Value) { return static_cast<char16_t>(BYTESWAP_ORDER16(static_cast<uint16>(Value))); }
