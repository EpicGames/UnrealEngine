// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformString.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Char.h"
#include "Misc/VarArgs.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Templates/IsValidVariadicFunctionArg.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "UObject/CoreReflectedTypeAccessors.h"

#define MAX_SPRINTF 1024

/** Determines case sensitivity options for string comparisons. */
namespace ESearchCase
{
	enum Type : int
	{
		/** Case sensitive. Upper/lower casing must match for strings to be considered equal. */
		CaseSensitive,

		/** Ignore case. Upper/lower casing does not matter when making a comparison. */
		IgnoreCase,
	};
};

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<ESearchCase::Type>();
#endif

/** Determines search direction for string operations. */
namespace ESearchDir
{
	enum Type : int
	{
		/** Search from the start, moving forward through the string. */
		FromStart,

		/** Search from the end, moving backward through the string. */
		FromEnd,
	};
}

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<ESearchDir::Type>();
#endif

/** Helper class used to convert CString into a boolean value. */
struct FToBoolHelper
{
	static CORE_API bool FromCStringAnsi( const ANSICHAR* String );
	static CORE_API bool FromCStringWide( const WIDECHAR* String );
	static CORE_API bool FromCStringUtf8( const UTF8CHAR* String );
};

/**
 *	Set of basic string utility functions operating on plain C strings. In addition to the 
 *	wrapped C string API,this struct also contains a set of widely used utility functions that
 *  operate on c strings.
 *	There is a specialized implementation for ANSICHAR and WIDECHAR strings provided. To access these
 *	functionality, the convenience typedefs FCString and FCStringAnsi are provided.
 **/
template <typename T>
struct TCString
{
	typedef T CharType;

	/**
	 * Returns whether this string contains only pure ansi characters 
	 * @param Str - string that will be checked
	 **/
	static bool IsPureAnsi(const CharType* Str)
	{
		if constexpr (std::is_same_v<CharType, ANSICHAR>)
		{
			return true;
		}
		else if constexpr (std::is_same_v<CharType, WIDECHAR>)
		{
			for (; *Str; Str++)
			{
				if (*Str > 0x7f)
				{
					return false;
				}
			}
			return true;
		}
		else if constexpr (std::is_same_v<CharType, UTF8CHAR>)
		{
			for (; *Str; Str++)
			{
				if ((uint8)*Str > 0x7f)
				{
					return false;
				}
			}
			return true;
		}
		else
		{
			static_assert(sizeof(CharType) == 0, "Not supported");
		}
	}

	static bool IsPureAnsi(const CharType* Str, const SIZE_T StrLen)
	{
		if constexpr (std::is_same_v<CharType, ANSICHAR>)
		{
			return true;
		}
		else if constexpr (std::is_same_v<CharType, WIDECHAR>)
		{
			for (SIZE_T Idx = 0; Idx < StrLen; Idx++, Str++)
			{
				if (*Str > 0x7f)
				{
					return false;
				}
			}
			return true;
		}
		else if constexpr (std::is_same_v<CharType, UTF8CHAR>)
		{
			for (SIZE_T Idx = 0; Idx < StrLen; Idx++, Str++)
			{
				if ((uint8)*Str > 0x7f)
				{
					return false;
				}
			}
			return true;
		}
		else
		{
			static_assert(sizeof(CharType) == 0, "Not supported");
		}
	}

	/**
	 * Returns whether this string contains only numeric characters 
	 * @param Str - string that will be checked
	 **/
	static bool IsNumeric(const CharType* Str)
	{
		if (*Str == '-' || *Str == '+')
		{
			Str++;
		}

		bool bHasDot = false;
		while (*Str != '\0')
		{
			if (*Str == '.')
			{
				if (bHasDot)
				{
					return false;
				}
				bHasDot = true;
			}
			else if (!TChar<CharType>::IsDigit(*Str))
			{
				return false;
			}
			
			++Str;
		}

		return true;
	}
	
	/**
	 * strcpy wrapper
	 *
	 * @param Dest - destination string to copy to
	 * @param Destcount - size of Dest in characters
	 * @param Src - source string
	 * @return destination string
	 */
	static UE_FORCEINLINE_HINT CharType* Strcpy( CharType* Dest, const CharType* Src );

	// This version was deprecated because Strcpy taking a size field does not match the CRT strcpy, and on some platforms the size field was being ignored.
	// If the memzeroing causes a performance problem, request a new function StrcpyTruncate.
	UE_DEPRECATED(5.6, "Use Strncpy instead. Note that Strncpy has a behavior difference from Strcpy: it memzeroes the entire DestCount-sized buffer after the end of string.")
	static inline CharType* Strcpy(CharType* Dest, SIZE_T DestCount, const CharType* Src);

	/** 
	 * Copy a string with length checking. Behavior differs from strncpy in that last character is zeroed. 
	 *
	 * @param Dest - destination buffer to copy to
	 * @param Src - source buffer to copy from
	 * @param MaxLen - max length of the buffer (including null-terminator)
	 * @return pointer to resulting string buffer
	 */
	static inline CharType* Strncpy( CharType* Dest, const CharType* Src, SIZE_T MaxLen );

	/**
	 * strcpy wrapper
	 * (templated version to automatically handle static destination array case)
	 *
	 * @param Dest - destination string to copy to
	 * @param Src - source string
	 * @return destination string
	 */
	template<SIZE_T DestCount>
	static UE_FORCEINLINE_HINT CharType* Strcpy( CharType (&Dest)[DestCount], const CharType* Src ) 
	{
		return Strncpy( Dest, Src, DestCount);
	}

	/**
	 * strcat wrapper
	 *
	 * @param Dest - destination string to copy to
	 * @param Destcount - size of Dest in characters
	 * @param Src - source string
	 * @return destination string
	 */
	static UE_FORCEINLINE_HINT CharType* Strcat( CharType* Dest, const CharType* Src );

	// This version was deprecated because Strcat taking a size field does not match the CRT strcat, and on some platforms the size field was being ignored.
	UE_DEPRECATED(5.6, "Use StrncatTruncateDest instead.")
	static inline CharType* Strcat(CharType* Dest, SIZE_T DestSize, const CharType* Src);

	/**
	 * strcat wrapper
	 * (templated version to automatically handle static destination array case)
	 *
	 * @param Dest - destination string to copy to
	 * @param Src - source string
	 * @return destination string
	 */
	template<SIZE_T DestCount>
	static UE_FORCEINLINE_HINT CharType* Strcat( CharType (&Dest)[DestCount], const CharType* Src ) 
	{ 
		return StrncatTruncateDest( Dest, DestCount, Src);
	}

	// This version was deprecated because it had a different interpretation of the length argument than CRT strncat.
	UE_DEPRECATED(5.6, "Use StrncatTruncateDest instead.")
	static inline CharType* Strncat( CharType* Dest, const CharType* Src, int32 DestSize )
	{
		return StrncatTruncateDest(Dest, DestSize, Src);
	}

	/**
	 * Append at most SrcLen characters from Src to Dest. Append a null terminator after the last character from Src.
	 * This version matches the behavior of CRT strncat: the size given is the maximum number to append from the Src str.
	 * If you instead want to specify the length of the destination buffer, call StrncatTruncateDest.
	 *
	 * @param Dest - destination buffer to append to
	 * @param Src - source buffer to copy from
	 * @param SrcLen - maximum number of characters to copy from Src. If Src length is less than SrcLen,
	 *        all characters up to the null terminator, and the null terminator, will be copied into Dest.
	 * @return pointer to resulting string buffer
	 */
	static inline CharType* StrncatTruncateSrc(CharType* Dest, const CharType* Src, int32 SrcLen)
	{
		if (SrcLen <= 0 || !*Src)
		{
			return Dest;
		}
		CharType* NewDest = Dest + Strlen(Dest);
		int32 AppendedCount = 0;
		do
		{
			*NewDest++ = *Src++;
			++AppendedCount;
		} while (AppendedCount < SrcLen && *Src);
		*NewDest = CharType(0);
		return Dest;
	}

	/**
	 * Concatenate a string with length checking. Unlike the CRT strncat, the size argument is interpreted
	 * as the size of the Dest buffer (which must include space for the null terminator) rather than the max
	 * number of Src characters to append.
	 *
	 * @param Dest - destination buffer to append to
	 * @param Src - source buffer to copy from
	 * @param DestSize - max size of the dest buffer, including null terminator. Max number of non-terminating
	 *        characters written into Dest is DestSize - 1.
	 * @return pointer to resulting string buffer
	 */
	static inline CharType* StrncatTruncateDest(CharType* Dest, int32 DestSize, const CharType* Src)
	{
		if (!*Src)
		{
			return Dest;
		}
		int32 Len = Strlen(Dest);
		if (Len + 1 >= DestSize)
		{
			return Dest;
		}
		CharType* NewDest = Dest + Len;
		do
		{
			*NewDest++ = *Src++;
			++Len;
		} while (*Src && Len + 1 < DestSize);
		*NewDest = CharType(0);
		return Dest;
	}

	/**
	 * strupr wrapper
	 *
	 * @param Dest - destination string to convert
	 * @param Destcount - size of Dest in characters
	 * @return destination string
	 */
	static UE_FORCEINLINE_HINT CharType* Strupr( CharType* Dest, SIZE_T DestCount );

	
	/**
	 * strupr wrapper
	 * (templated version to automatically handle static destination array case)
	 *
	 * @param Dest - destination string to convert
	 * @return destination string
	 */
	template<SIZE_T DestCount>
	static UE_FORCEINLINE_HINT CharType* Strupr( CharType (&Dest)[DestCount] ) 
	{
		return Strupr( Dest, DestCount );
	}

	/**
	 * strcmp wrapper
	 **/
	static UE_FORCEINLINE_HINT int32 Strcmp( const CharType* String1, const CharType* String2 );

	/**
	 * strncmp wrapper
	 */
	static UE_FORCEINLINE_HINT int32 Strncmp( const CharType* String1, const CharType* String2, SIZE_T Count);

	/**
	 * stricmp wrapper
	 */
	static UE_FORCEINLINE_HINT int32 Stricmp( const CharType* String1, const CharType* String2 );
	
	/**
	 * strnicmp wrapper
	 */
	static UE_FORCEINLINE_HINT int32 Strnicmp( const CharType* String1, const CharType* String2, SIZE_T Count );

	/**
	 * Returns a static string that is filled with a variable number of spaces
	 *
	 * @param NumSpaces Number of spaces to put into the string, max of 255
	 * 
	 * @return The string of NumSpaces spaces.
	 */
	static const CharType* Spc( int32 NumSpaces );

	/**
	 * Returns a static string that is filled with a variable number of tabs
	 *
	 * @param NumTabs Number of tabs to put into the string, max of 255
	 * 
	 * @return The string of NumTabs tabs.
	 */
	static const CharType* Tab( int32 NumTabs );

	/**
	 * Find string in string, case sensitive, requires non-alphanumeric lead-in.
	 */
	static const CharType* Strfind( const CharType* Str, const CharType* Find, bool bSkipQuotedChars = false );

	/**
	 * Find string in string, case insensitive, requires non-alphanumeric lead-in.
	 */
	static const CharType* Strifind( const CharType* Str, const CharType* Find, bool bSkipQuotedChars = false );

	/**
	 * Finds string in string, case insensitive, requires the string be surrounded by one the specified
	 * delimiters, or the start or end of the string.
	 */
	static const CharType* StrfindDelim(const CharType* Str, const CharType* Find, const CharType* Delim=LITERAL(CharType, " \t,"));

	/** 
	 * Finds string in string, case insensitive 
	 * @param Str The string to look through
	 * @param Find The string to find inside Str
	 * @return Position in Str if Find was found, otherwise, nullptr. If Find is non-null but empty, returns Str.
	 */
	static const CharType* Stristr(const CharType* Str, const CharType* Find);

	/** 
	 * Finds string in string, case insensitive (non-const version)
	 * @param Str The string to look through
	 * @param Find The string to find inside Str
	 * @return Position in Str if Find was found, otherwise, nullptr. If Find is non-null but empty, returns Str.
	 */
	static CharType* Stristr(CharType* Str, const CharType* Find) { return (CharType*)Stristr((const CharType*)Str, Find); }

	/**
	 * Finds string in string, case insensitive
	 * @param Str The character array to look through
	 * @param InStrLen The length of the Str array
	 * @param Find The character array to find inside Str
	 * @param FindLen The length of the Find array
	 * @return Position in Str if Find was found, otherwise, nullptr. If FindLen is 0, returns Str.
	 */
	static const CharType* Strnistr(const CharType* Str, int32 InStrLen, const CharType* Find, int32 FindLen);

	/**
	 * Finds string in string, case insensitive (non-const version)
	 * @param Str The character array to look through
	 * @param InStrLen The length of the Str array
	 * @param Find The character array to find inside Str
	 * @param FindLen The length of the Find array
	 * @return Position in Str if Find was found, otherwise, nullptr. If FindLen is 0, returns Str.
	 */
	static CharType* Strnistr(CharType* Str, int32 InStrLen, const CharType* Find, int32 FindLen)
	{
		return (CharType*)Strnistr((const CharType*)Str, InStrLen, Find, FindLen);
	}

	/**
	 * Finds string in string, case sensitive
	 * @param Str The character array to look through
	 * @param InStrLen The length of the Str array
	 * @param Find The character array to find inside Str
	 * @param FindLen The length of the Find array
	 * @return Position in Str if Find was found, otherwise, nullptr. If FindLen is 0, returns Str.
	 */
	static const CharType* Strnstr(const CharType* Str, int32 InStrLen, const CharType* Find, int32 FindLen);

	/**
	 * Finds string in string, case sensitive (non-const version)
	 * @param Str The character array to look through
	 * @param InStrLen The length of the Str array
	 * @param Find The character array to find inside Str
	 * @param FindLen The length of the Find array
	 * @return Position in Str if Find was found, otherwise, nullptr. If FindLen is 0, returns Str.
	 */
	static CharType* Strnstr(CharType* Str, int32 InStrLen, const CharType* Find, int32 FindLen)
	{
		return (CharType*)Strnstr((const CharType*)Str, InStrLen, Find, FindLen);
	}

	/**
	 * strlen wrapper
	 */
	static inline int32 Strlen( const CharType* String );

	/**
	 * Calculate the length of the string up to the given size.
	 * @param String A possibly-null-terminated string in a character array with a size of at least StringSize.
	 * @param StringSize The maximum number of characters to read from String.
	 * @return Length The smaller of StringSize and the number of characters in String before a null character.
	 */
	static UE_FORCEINLINE_HINT int32 Strnlen( const CharType* String, SIZE_T StringSize );

	/**
	 * strstr wrapper
	 */
	static UE_FORCEINLINE_HINT const CharType* Strstr( const CharType* String, const CharType* Find );
	static UE_FORCEINLINE_HINT CharType* Strstr( CharType* String, const CharType* Find );

	/**
	 * strchr wrapper
	 */
	static UE_FORCEINLINE_HINT const CharType* Strchr( const CharType* String, CharType c );
	static UE_FORCEINLINE_HINT CharType* Strchr( CharType* String, CharType c );

	/**
	 * strrchr wrapper
	 */
	static UE_FORCEINLINE_HINT const CharType* Strrchr( const CharType* String, CharType c );
	static UE_FORCEINLINE_HINT CharType* Strrchr( CharType* String, CharType c );

	/**
	 * strrstr wrapper
	 */
	static UE_FORCEINLINE_HINT const CharType* Strrstr( const CharType* String, const CharType* Find );
	static inline CharType* Strrstr( CharType* String, const CharType* Find );

	/**
	 * strspn wrapper
	 */
	static inline int32 Strspn( const CharType* String, const CharType* Mask );

	/**
	 * strcspn wrapper
	 */
	static inline int32 Strcspn( const CharType* String, const CharType* Mask );

	/**
	 * atoi wrapper
	 */
	static UE_FORCEINLINE_HINT int32 Atoi( const CharType* String );

	/**
	 * atoi64 wrapper
	 */
	static UE_FORCEINLINE_HINT int64 Atoi64( const CharType* String );
	
	/**
	 * atof wrapper
	 */
	static UE_FORCEINLINE_HINT float Atof( const CharType* String );

	/**
	 * atod wrapper
	 */
	static UE_FORCEINLINE_HINT double Atod( const CharType* String );

	/**
	 * Converts a string into a boolean value
	 *   1, "True", "Yes", FCoreTexts::True, FCoreTexts::Yes, and non-zero integers become true
	 *   0, "False", "No", FCoreTexts::False, FCoreTexts::No, and unparsable values become false
	 *
	 * @return The boolean value
	 */
	static bool ToBool(const CharType* String)
	{
		if constexpr (std::is_same_v<CharType, ANSICHAR>)
		{
			return FToBoolHelper::FromCStringAnsi(String);
		}
		else if constexpr (std::is_same_v<CharType, WIDECHAR>)
		{
			return FToBoolHelper::FromCStringWide(String);
		}
		else if constexpr (std::is_same_v<CharType, UTF8CHAR>)
		{
			return FToBoolHelper::FromCStringUtf8(String);
		}
		else
		{
			static_assert(sizeof(CharType) == 0, "Not supported");
			return false;
		}
	}

	/**
	 * strtoi wrapper
	 */
	static UE_FORCEINLINE_HINT int32 Strtoi( const CharType* Start, CharType** End, int32 Base );

	/**
	 * strtoi wrapper
	 */
	static UE_FORCEINLINE_HINT int64 Strtoi64( const CharType* Start, CharType** End, int32 Base );

	/**
	 * strtoui wrapper
	 */
	static UE_FORCEINLINE_HINT uint64 Strtoui64( const CharType* Start, CharType** End, int32 Base );

	/**
	 * strtok wrapper
	 */
	static UE_FORCEINLINE_HINT CharType* Strtok( CharType* TokenString, const CharType* Delim, CharType** Context );

private:
	static int32 VARARGS SprintfImpl(CharType* Dest, const CharType* Fmt, ...);
	static int32 VARARGS SnprintfImpl(CharType* Dest, int32 DestSize, const CharType* Fmt, ...);

	template <typename SrcEncoding>
	using TIsCharEncodingCompatibleWithCharType = TIsCharEncodingCompatibleWith<SrcEncoding, CharType>;

public:
	/**
	* Standard string formatted print. 
	* @warning: make sure code using FCString::Sprintf allocates enough (>= MAX_SPRINTF) memory for the destination buffer
	*/
	template <typename FmtType, typename... Types>
	static int32 Sprintf(CharType* Dest, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithCharType>::Value, "Formatting string must be a literal string of a char type compatible with the TCString type.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to TCString::Sprintf");

		return SprintfImpl(Dest, (const CharType*)Fmt, Args...);
	}

	/** 
	 * Safe string formatted print. 
	 */
	template <typename FmtType, typename... Types>
	static int32 Snprintf(CharType* Dest, int32 DestSize, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithCharType>::Value, "Formatting string must be a literal string of a char type compatible with the TCString type.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to TCString::Snprintf");

		return SnprintfImpl(Dest, DestSize, (const CharType*)Fmt, Args...);
	}

	/**
	 * Format to a destination buffer. Similar to vsnprintf.
	 *
	 * Writes the formatted output to Dest, up to a maximum of DestSize-1 characters, excluding the null terminator.
	 * Content of Dest is undefined when the return value is outside of the range [0, DestSize).
	 * A return value is calculated even when DestSize is 0, in which case Dest may be null.
	 * A null terminator is always written unless DestSize is 0.
	 * Success is indicated by ReturnValue >= 0 && ReturnValue < DestSize.
	 *
	 * %hs is always treated as UTF-8.
	 * %ls is always treated as UTF-16.
	 * %s is treated as UTF-8 for narrow format strings and UTF-16 for wide format strings.
	 *
	 * @param Dest   Pointer to the destination buffer to write to.
	 * @param DestSize   Maximum number of characters to write to Dest, including the null terminator.
	 * @param Format   Format string with which to format the output.
	 * @param Args   Arguments containing the data referenced by the format string.
	 * @return Number of characters, excluding the null terminator, that would be
	 *         written to Dest if DestSize was ignored, or -1 on error.
	 */
	UE_REWRITE static int32 FormatV(CharType* Dest, SIZE_T DestSize, const CharType* Format, va_list Args)
	{
		return FPlatformString::FormatV(Dest, DestSize, Format, Args);
	}

	/**
	 * Helper function to write formatted output using an argument list
	 *
	 * @param Dest - destination string buffer
	 * @param DestSize - size of destination buffer
	 * @param Fmt - string to print
	 * @param Args - argument list
	 * @return number of characters written or -1 if truncated
	 */
	static UE_FORCEINLINE_HINT int32 GetVarArgs( CharType* Dest, SIZE_T DestSize, const CharType*& Fmt, va_list ArgPtr );
};

typedef TCString<TCHAR>    FCString;
typedef TCString<ANSICHAR> FCStringAnsi;
typedef TCString<WIDECHAR> FCStringWide;
typedef TCString<UTF8CHAR> FCStringUtf8;

/*-----------------------------------------------------------------------------
	generic TCString implementations
-----------------------------------------------------------------------------*/

template <typename CharType = TCHAR>
struct TCStringSpcHelper
{
	/** Number of characters to be stored in string. */
	static constexpr int32 MAX_SPACES = 255;

	/** Number of tabs to be stored in string. */
	static constexpr int32 MAX_TABS = 255;

	static CORE_API const CharType SpcArray[MAX_SPACES + 1];
	static CORE_API const CharType TabArray[MAX_TABS + 1];
};

template <typename T>
const typename TCString<T>::CharType* TCString<T>::Spc( int32 NumSpaces )
{
	check(NumSpaces >= 0 && NumSpaces <= TCStringSpcHelper<T>::MAX_SPACES);
	return TCStringSpcHelper<T>::SpcArray + TCStringSpcHelper<T>::MAX_SPACES - NumSpaces;
}

template <typename T>
const typename TCString<T>::CharType* TCString<T>::Tab( int32 NumTabs )
{
	check(NumTabs >= 0 && NumTabs <= TCStringSpcHelper<T>::MAX_TABS);
	return TCStringSpcHelper<T>::TabArray + TCStringSpcHelper<T>::MAX_TABS - NumTabs;
}

//
// Find string in string, case sensitive, requires non-alphanumeric lead-in.
//
template <typename T>
const typename TCString<T>::CharType* TCString<T>::Strfind(const CharType* Str, const CharType* Find, bool bSkipQuotedChars)
{
	if (Find == NULL || Str == NULL)
	{
		return NULL;
	}

	bool Alnum = 0;
	CharType f = *Find;
	int32 Length = Strlen(Find++) - 1;
	CharType c = *Str++;
	if (bSkipQuotedChars)
	{
		bool bInQuotedStr = false;
		while (c)
		{
			if (!bInQuotedStr && !Alnum && c == f && !Strncmp(Str, Find, Length))
			{
				return Str - 1;
			}
			Alnum = (c >= LITERAL(CharType, 'A') && c <= LITERAL(CharType, 'Z')) ||
				(c >= LITERAL(CharType, 'a') && c <= LITERAL(CharType, 'z')) ||
				(c >= LITERAL(CharType, '0') && c <= LITERAL(CharType, '9'));
			if (c == LITERAL(CharType, '"'))
			{
				bInQuotedStr = !bInQuotedStr;
			}
			c = *Str++;
		}
	}
	else
	{
		while (c)
		{
			if (!Alnum && c == f && !Strncmp(Str, Find, Length))
			{
				return Str - 1;
			}
			Alnum = (c >= LITERAL(CharType, 'A') && c <= LITERAL(CharType, 'Z')) ||
				(c >= LITERAL(CharType, 'a') && c <= LITERAL(CharType, 'z')) ||
				(c >= LITERAL(CharType, '0') && c <= LITERAL(CharType, '9'));
			c = *Str++;
		}
	}
	return NULL;
}

//
// Find string in string, case insensitive, requires non-alphanumeric lead-in.
//
template <typename T>
const typename TCString<T>::CharType* TCString<T>::Strifind( const CharType* Str, const CharType* Find, bool bSkipQuotedChars )
{
	if( Find == NULL || Str == NULL )
	{
		return NULL;
	}
	
	bool Alnum  = 0;
	CharType f = ( *Find < LITERAL(CharType, 'a') || *Find > LITERAL(CharType, 'z') ) ? (*Find) : (CharType)(*Find + LITERAL(CharType,'A') - LITERAL(CharType,'a'));
	int32 Length = Strlen(Find++)-1;
	CharType c = *Str++;
	
	if (bSkipQuotedChars)
	{
		bool bInQuotedStr = false;
		while( c )
		{
			if( c >= LITERAL(CharType, 'a') && c <= LITERAL(CharType, 'z') )
			{
				c += LITERAL(CharType, 'A') - LITERAL(CharType, 'a');
			}
			if( !bInQuotedStr && !Alnum && c==f && !Strnicmp(Str,Find,Length) )
			{
				return Str-1;
			}
			Alnum = (c>=LITERAL(CharType,'A') && c<=LITERAL(CharType,'Z')) || 
					(c>=LITERAL(CharType,'0') && c<=LITERAL(CharType,'9'));
			if (c == LITERAL(CharType, '"'))
			{
				bInQuotedStr = !bInQuotedStr;
			}
			c = *Str++;
		}
	}
	else
	{
		while( c )
		{
			if( c >= LITERAL(CharType, 'a') && c <= LITERAL(CharType, 'z') )
			{
				c += LITERAL(CharType, 'A') - LITERAL(CharType, 'a');
			}
			if( !Alnum && c==f && !Strnicmp(Str,Find,Length) )
			{
				return Str-1;
			}
			Alnum = (c>=LITERAL(CharType,'A') && c<=LITERAL(CharType,'Z')) || 
					(c>=LITERAL(CharType,'0') && c<=LITERAL(CharType,'9'));
			c = *Str++;
		}
	}
	return NULL;
}

//
// Finds string in string, case insensitive, requires the string be surrounded by one the specified
// delimiters, or the start or end of the string.
//
template <typename T>
const typename TCString<T>::CharType* TCString<T>::StrfindDelim(const CharType* Str, const CharType* Find, const CharType* Delim)
{
	if( Find == NULL || Str == NULL )
	{
		return NULL;
	}

	int32 Length = Strlen(Find);
	const T* Found = Stristr(Str, Find);
	if( Found )
	{
		// check if this occurrence is delimited correctly
		if( (Found==Str || Strchr(Delim, Found[-1])!=NULL) &&								// either first char, or following a delim
			(Found[Length]==LITERAL(CharType,'\0') || Strchr(Delim, Found[Length])!=NULL) )	// either last or with a delim following
		{
			return Found;
		}

		// start searching again after the first matched character
		for(;;)
		{
			Str = Found+1;
			Found = Stristr(Str, Find);

			if( Found == NULL )
			{
				return NULL;
			}

			// check if the next occurrence is delimited correctly
			if( (Strchr(Delim, Found[-1])!=NULL) &&												// match is following a delim
				(Found[Length]==LITERAL(CharType,'\0') || Strchr(Delim, Found[Length])!=NULL) )	// either last or with a delim following
			{
				return Found;
			}
		}
	}

	return NULL;
}

/** 
 * Finds string in string, case insensitive 
 * @param Str The string to look through
 * @param Find The string to find inside Str
 * @return Position in Str if Find was found, otherwise, NULL. If Find is non-null but empty, returns Str.
 */
template <typename T>
const typename TCString<T>::CharType* TCString<T>::Stristr(const CharType* Str, const CharType* Find)
{
	// Note this implementation is not reducible to Strnistr because we need to avoid calling strlen(Str) to
	// avoid scanning it twice

	// both strings must be valid
	if( Find == nullptr || Str == nullptr )
	{
		return nullptr;
	}

	// get upper-case first letter of the find string (to reduce the number of full strnicmps)
	CharType FindInitial = TChar<CharType>::ToUpper(*Find);
	if (!FindInitial)
	{
		// When searching for the empty string, always return index of the first element of Str even if Str is empty.
		return Str;
	}
	// get length of find string, and increment past first letter
	int32   Length = Strlen(Find++) - 1;
	// get the first letter of the search string, and increment past it
	CharType StrChar = *Str++;
	// while we aren't at end of string...
	while (StrChar)
	{
		// make sure it's upper-case
		StrChar = TChar<CharType>::ToUpper(StrChar);
		// if it matches the first letter of the find string, do a case-insensitive string compare for the length of the find string
		if (StrChar == FindInitial && !Strnicmp(Str, Find, Length))
		{
			// if we found the string, then return a pointer to the beginning of it in the search string
			return Str-1;
		}
		// go to next letter
		StrChar = *Str++;
	}

	// if nothing was found, return nullptr
	return nullptr;
}

template <typename T>
const typename TCString<T>::CharType* TCString<T>::Strnistr(const CharType* Str, int32 InStrLen, const CharType* Find, int32 FindLen)
{
	if (FindLen <= 0)
	{
		checkf(FindLen >= 0, TEXT("Invalid FindLen: %d"), FindLen);
		return Str;
	}
	if (InStrLen < FindLen)
	{
		checkf(InStrLen >= 0, TEXT("Invalid InStrLen: %d"), InStrLen);
		return nullptr;
	}
	if (Find == nullptr || Str == nullptr)
	{
		return nullptr;
	}

	// get upper-case first letter of the find string (to reduce the number of full strnicmps)
	CharType FindInitial = TChar<CharType>::ToUpper(*Find);
	// Set FindSuffix,FindSuffixLength to the characters of Find after the first letter
	int32 FindSuffixLength = FindLen - 1;
	const CharType* FindSuffix = Find + 1;

	// while the length of the remaining string is >= FindLen
	const CharType* StrLastChance = Str + InStrLen - FindLen;
	while (Str <= StrLastChance)
	{
		CharType StrChar = *Str++;

		// make sure it's upper-case
		StrChar = TChar<CharType>::ToUpper(StrChar);
		// if it matches the first letter of the find string, do a case-insensitive string compare for the length of the find string
		if (StrChar == FindInitial && !Strnicmp(Str, FindSuffix, FindSuffixLength))
		{
			// if we found the string, then return a pointer to the beginning of it in the search string
			return Str - 1;
		}
	}

	// if nothing was found, return nullptr
	return nullptr;
}

template <typename T>
const typename TCString<T>::CharType* TCString<T>::Strnstr(const CharType* Str, int32 InStrLen, const CharType* Find, int32 FindLen)
{
	if (FindLen <= 0)
	{
		checkf(FindLen >= 0, TEXT("Invalid FindLen: %d"), FindLen);
		return Str;
	}
	if (InStrLen < FindLen)
	{
		checkf(InStrLen >= 0, TEXT("Invalid InStrLen: %d"), InStrLen);
		return nullptr;
	}
	if (Find == nullptr || Str == nullptr)
	{
		return nullptr;
	}

	// get first letter of the find string (to reduce the number of full strncmps)
	const CharType FindInitial = *Find;
	// Set FindSuffix,FindSuffixLength to the characters of Find after the first letter
	const int32 FindSuffixLength = FindLen - 1;
	const CharType* FindSuffix = Find + 1;

	// while the length of the remaining string is >= FindLen
	const CharType* StrLastChance = Str + InStrLen - FindLen;
	
	if constexpr (sizeof(CharType) >= 1 && sizeof(CharType) <= 2)
	{
		constexpr uint64 SignBit    = sizeof(CharType) == 1 ? 0x80ull : 0x8000ull;
		constexpr uint64 Mask       = sizeof(CharType) == 1 ? 0xFFull : 0xFFFFull;
		constexpr uint64 Ones64     = sizeof(CharType) == 1 ? 0x0101010101010101ull : 0x0001000100010001ull;
		constexpr uint64 Positive64 = sizeof(CharType) == 1 ? 0x7F7F7F7F7F7F7F7Full : 0x7FFF7FFF7FFF7FFFull;
		constexpr uint64 SignBit64  = sizeof(CharType) == 1 ? 0x8080808080808080ull : 0x8000800080008000ull;
		constexpr uint64 CharPer64  = sizeof(uint64) / sizeof(CharType);
		constexpr uint64 BitPerChar = sizeof(CharType) == 1 ? 8 : 16;

		InStrLen -= FindSuffixLength;
		if (InStrLen >= CharPer64)
		{
			// Process normally if we're unaligned
			for (; (UPTRINT(Str) & (BitPerChar-1)) && InStrLen > 1; --InStrLen)
			{
				CharType StrChar = *Str++;

				// if it matches the first letter of the find string, do a string compare for the length of the find string
				if (StrChar == FindInitial && !TCString<CharType>::Strncmp(Str, FindSuffix, FindSuffixLength))
				{
					// if we found the string, then return a pointer to the beginning of it in the search string
					return Str - 1;
				}
			}

			// Broadcast the initial value in each slots.
			const uint64 FindInitial64 = (uint64)FindInitial * Ones64;

			for (; InStrLen >= (int32)CharPer64; Str += CharPer64, InStrLen -= (int32)CharPer64)
			{
				uint64 StrValue = *(const uint64*)Str;
				// We want to test each slots perfect match with the initial character
				// The trick is to avoid overflowing a slot into another one.
				// So we XOR and NEGATE so we end up with 0xFF when the character match.
				// Then we remove the sign bit with 0x7F and ADD one, which will end up with 0x80
				// if all the bits are set. We also AND with 0x80 to test the sign bit separately.
				// And then we AND those two things together that will end up with 0x80 for each
				// slot where there is a match.
				StrValue = ~(StrValue ^ FindInitial64);
				StrValue = ((StrValue & Positive64) + Ones64) & (StrValue & SignBit64);

				// If any slot has a match, begin analysis
				if (StrValue)
				{
					for (const CharType* InnerStr = Str; StrValue; StrValue >>= BitPerChar, ++InnerStr)
					{
						if ((StrValue & Mask) == SignBit && !TCString<CharType>::Strncmp(InnerStr + 1, FindSuffix, FindSuffixLength))
						{
							return InnerStr;
						}
					}
				}
			}
		}
	}

	while (Str <= StrLastChance)
	{
		CharType StrChar = *Str++;

		// if it matches the first letter of the find string, do a string compare for the length of the find string
		if (StrChar == FindInitial && !Strncmp(Str, FindSuffix, FindSuffixLength))
		{
			// if we found the string, then return a pointer to the beginning of it in the search string
			return Str - 1;
		}
	}

	// if nothing was found, return nullptr
	return nullptr;
}

template <typename T> UE_FORCEINLINE_HINT
typename TCString<T>::CharType* TCString<T>::Strcpy(CharType* Dest, const CharType* Src)
{
	return FPlatformString::Strcpy(Dest, Src);
}

template <typename T> inline
typename TCString<T>::CharType* TCString<T>::Strcpy(CharType* Dest, SIZE_T DestCount, const CharType* Src)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return FPlatformString::Strcpy(Dest, DestCount, Src);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

template <typename T> inline
typename TCString<T>::CharType* TCString<T>::Strncpy( CharType* Dest, const CharType* Src, SIZE_T MaxLen )
{
	FPlatformString::Strncpy(Dest, Src, MaxLen);
	return Dest;
}

template <typename T> UE_FORCEINLINE_HINT
typename TCString<T>::CharType* TCString<T>::Strcat(CharType* Dest, const CharType* Src)
{
	return FPlatformString::Strcat(Dest, Src);
}

template <typename T> inline
typename TCString<T>::CharType* TCString<T>::Strcat(CharType* Dest, SIZE_T DestCount, const CharType* Src)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return FPlatformString::Strcat(Dest, DestCount, Src);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

template <typename T> UE_FORCEINLINE_HINT
typename TCString<T>::CharType* TCString<T>::Strupr(CharType* Dest, SIZE_T DestCount)
{
	return FPlatformString::Strupr(Dest, DestCount);
}

template <typename T> UE_FORCEINLINE_HINT
int32 TCString<T>::Strcmp( const CharType* String1, const CharType* String2 )
{
	return FPlatformString::Strcmp(String1, String2);
}

template <typename T> UE_FORCEINLINE_HINT
int32 TCString<T>::Strncmp( const CharType* String1, const CharType* String2, SIZE_T Count)
{
	return FPlatformString::Strncmp(String1, String2, Count);
}

template <typename T> UE_FORCEINLINE_HINT
int32 TCString<T>::Stricmp( const CharType* String1, const CharType* String2 )
{
	return FPlatformString::Stricmp(String1, String2);
}

template <typename T> UE_FORCEINLINE_HINT
int32 TCString<T>::Strnicmp( const CharType* String1, const CharType* String2, SIZE_T Count )
{
	return FPlatformString::Strnicmp(String1, String2, Count);
}

namespace UE::Core::Private
{
	CORE_API int32 Strlen32(const UTF32CHAR* String);
}

template <typename T>
UE_NODEBUG inline int32 TCString<T>::Strlen( const CharType* String ) 
{
	if constexpr (std::is_same_v<T, UTF32CHAR>)
	{
		return UE::Core::Private::Strlen32(String);
	}
	else
	{
		return FPlatformString::Strlen(String);
	}
}

template <typename T> UE_FORCEINLINE_HINT
int32 TCString<T>::Strnlen( const CharType* String, SIZE_T StringSize ) 
{
	return FPlatformString::Strnlen(String, StringSize);
}

template <typename T> UE_FORCEINLINE_HINT
const typename TCString<T>::CharType* TCString<T>::Strstr( const CharType* String, const CharType* Find )
{
	return FPlatformString::Strstr(String, Find);
}

template <typename T> UE_FORCEINLINE_HINT
typename TCString<T>::CharType* TCString<T>::Strstr( CharType* String, const CharType* Find )
{
	return (CharType*)FPlatformString::Strstr(String, Find);
}

template <typename T> UE_FORCEINLINE_HINT
const typename TCString<T>::CharType* TCString<T>::Strchr( const CharType* String, CharType c ) 
{ 
	return FPlatformString::Strchr(String, c);
}

template <typename T> UE_FORCEINLINE_HINT
typename TCString<T>::CharType* TCString<T>::Strchr( CharType* String, CharType c ) 
{ 
	return (CharType*)FPlatformString::Strchr(String, c);
}

template <typename T> UE_FORCEINLINE_HINT
const typename TCString<T>::CharType* TCString<T>::Strrchr( const CharType* String, CharType c )
{ 
	return FPlatformString::Strrchr(String, c);
}

template <typename T> UE_FORCEINLINE_HINT
typename TCString<T>::CharType* TCString<T>::Strrchr( CharType* String, CharType c )
{ 
	return (CharType*)FPlatformString::Strrchr(String, c);
}

template <typename T> UE_FORCEINLINE_HINT
const typename TCString<T>::CharType* TCString<T>::Strrstr( const CharType* String, const CharType* Find )
{
	return Strrstr((CharType*)String, Find);
}

template <typename T> inline
typename TCString<T>::CharType* TCString<T>::Strrstr( CharType* String, const CharType* Find )
{
	if (String == nullptr || Find == nullptr)
	{
		return nullptr;
	}
	if (*Find == (CharType)0)
	{
		return String + Strlen(String);
	}

	CharType* Result = nullptr;
	for (;;)
	{
		CharType* Found = Strstr(String, Find);
		if (!Found)
		{
			return Result;
		}

		Result = Found;
		String = Found + 1;
	}
}

template <typename T> inline
int32 TCString<T>::Strspn( const CharType* String, const CharType* Mask )
{
	const CharType* StringIt = String;
	while (*StringIt)
	{
		for (const CharType* MaskIt = Mask; *MaskIt; ++MaskIt)
		{
			if (*StringIt == *MaskIt)
			{
				goto NextChar;
			}
		}

		return UE_PTRDIFF_TO_INT32(StringIt - String);

	NextChar:
		++StringIt;
	}

	return UE_PTRDIFF_TO_INT32(StringIt - String);
}

template <typename T> inline
int32 TCString<T>::Strcspn( const CharType* String, const CharType* Mask )
{
	const CharType* StringIt = String;
	while (*StringIt)
	{
		for (const CharType* MaskIt = Mask; *MaskIt; ++MaskIt)
		{
			if (*StringIt == *MaskIt)
			{
				return UE_PTRDIFF_TO_INT32(StringIt - String);
			}
		}

		++StringIt;
	}

	return UE_PTRDIFF_TO_INT32(StringIt - String);
}

template <typename T> UE_FORCEINLINE_HINT 
int32 TCString<T>::Atoi( const CharType* String ) 
{
	return FPlatformString::Atoi(String);
}

template <typename T> UE_FORCEINLINE_HINT
int64 TCString<T>::Atoi64( const CharType* String )
{ 
	return FPlatformString::Atoi64(String);
}

template <typename T> UE_FORCEINLINE_HINT
float TCString<T>::Atof( const CharType* String )
{ 
	return FPlatformString::Atof(String);
}

template <typename T> UE_FORCEINLINE_HINT
double TCString<T>::Atod( const CharType* String )
{ 
	return FPlatformString::Atod(String);
}

template <typename T> UE_FORCEINLINE_HINT
int32 TCString<T>::Strtoi( const CharType* Start, CharType** End, int32 Base ) 
{ 
	return FPlatformString::Strtoi(Start, End, Base);
}

template <typename T> UE_FORCEINLINE_HINT
int64 TCString<T>::Strtoi64( const CharType* Start, CharType** End, int32 Base ) 
{ 
	return FPlatformString::Strtoi64(Start, End, Base);
}

template <typename T> UE_FORCEINLINE_HINT
uint64 TCString<T>::Strtoui64( const CharType* Start, CharType** End, int32 Base ) 
{ 
	return FPlatformString::Strtoui64(Start, End, Base);
}


template <typename T> UE_FORCEINLINE_HINT
typename TCString<T>::CharType* TCString<T>::Strtok( CharType* TokenString, const CharType* Delim, CharType** Context )
{ 
	return FPlatformString::Strtok(TokenString, Delim, Context);
}

template <typename T> UE_FORCEINLINE_HINT
int32 TCString<T>::GetVarArgs( CharType* Dest, SIZE_T DestSize, const CharType*& Fmt, va_list ArgPtr )
{
	return FPlatformString::GetVarArgs(Dest, DestSize, Fmt, ArgPtr);
}

template <typename T>
int32 TCString<T>::SprintfImpl(CharType* Dest, const CharType* Fmt, ...)
{
	static_assert(std::is_same_v<CharType, ANSICHAR> || std::is_same_v<CharType, WIDECHAR> || std::is_same_v<CharType, UTF8CHAR>, "Not supported");

	int32	Result = -1;
	GET_TYPED_VARARGS_RESULT(CharType, Dest, MAX_SPRINTF, MAX_SPRINTF - 1, Fmt, Fmt, Result);
	return Result;
}

template <typename T>
int32 TCString<T>::SnprintfImpl(CharType* Dest, int32 DestSize, const CharType* Fmt, ...)
{
	static_assert(std::is_same_v<CharType, ANSICHAR> || std::is_same_v<CharType, WIDECHAR> || std::is_same_v<CharType, UTF8CHAR>, "Not supported");

	int32	Result = -1;
	GET_TYPED_VARARGS_RESULT(CharType, Dest, DestSize, DestSize - 1, Fmt, Fmt, Result);
	return Result;
}
