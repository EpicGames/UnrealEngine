// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

class FFeedbackContext;
class FReferenceCollector;
class UObject;

/**
 * FChange modifies a UObject and is meant to be used to implement undo/redo.
 * The change is embedded in an FTransaction which executes it *instead* of the standard 
 * serialization transaction (cannot be combined - see FTransaction).
 *
 * The original FChange style (used by MeshEditor) was that calling Execute() would return a new 
 * FChange that applies the opposite action, and FTransaction would swap the two at each undo/redo
 * step (eg a "DeleteObject" FChange would return a "CreateObject" FChange)
 *
 * The alternative "Command Pattern"-style FChange calls Apply() and Revert() on a single FChange.
  *
 * FChange may eventually be deprecated. You should subclass 
 * FSwapChange and FCommandChange to implement these different styles.
 */
class FChange
{

public:
	enum class EChangeStyle
	{
		InPlaceSwap,			// Call Execute() which returns new "opposite" FChange   (default)
		CommandPattern			// call Revert() to Undo and Apply() to Redo
	};

	/** What style of change is this */
	virtual EChangeStyle GetChangeType() = 0;

	/** Makes the change to the object, returning a new change that can be used to perfectly roll back this change */
	virtual TUniquePtr<FChange> Execute( UObject* Object ) = 0;

	/** Makes the change to the object */
	virtual void Apply( UObject* Object ) = 0;

	/** Reverts change to the object */
	virtual void Revert( UObject* Object ) = 0;

	/** @return true if this Change has Expired, ie it will no longer have any effect and could be skipped by undo/redo */
	virtual bool HasExpired( UObject* Object ) const { return false; }

	/** Used by GC to collect referenced objects. */
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) { }

	/** Describes this change (for debugging) */
	virtual FString ToString() const = 0;

	/** Prints this change to the log, including sub-changes if there are any.  For compound changes, there might be multiple lines.  You should not need to override this function. */
	CORE_API virtual void PrintToLog( FFeedbackContext& FeedbackContext, const int32 IndentLevel = 0 );

	/**
	 * Gets the approximate number of bytes this change takes up in memory.
	 *
	 * Used to
	 *	- determine the size of transactions based on a custom change, e.g. when trimming the oldest transactions when the undo stack size is exceeded.
	 *	- displaying the stack size, e.g. in the Undo Panel.
	 *
	 * A typical implementation returns the size of any dynamically allocated memory, such GetAllocatedSize called on its containers (TArray, TSet, TMap).
	 * You should consider adding sizeof(FMyChange), if it's not insignificant. Remember: it's enough to return the approx. allocation size; it's
	 * unreasonable to expect every FChange implementation to return the exact amount, which also is not needed for the purpose of this function.
	 * Keep in mind, that for nested containers you need to call GetAllocatedSize recursively, e.g. for a "TArray<TArray<T>> Array" you'd need to
	 * return "Array.GetAllocatedSize() + Algo::Accumulate(Array, 0, [](const TArray<T>& A){ return A.GetAllocatedSize(); })".
	 *
	 * This function is introduced in 5.7: too many FChange implementations exist so this function will need to be implemented gradually.
	 * 
	 * @return The approximate number of bytes this change takes up in memory.
	 */
	virtual SIZE_T GetSize() const { return 0; }

	/** Virtual destructor */
	virtual ~FChange() = default;

protected:

	/** Protected default constructor */
	FChange()
	{
	}

private:
	// Non-copyable
	FChange( const FChange& ) = delete;
	FChange& operator=( const FChange& ) = delete;

};



/**
 * To use FSwapChange you must implement Execute().
 * This function must do two things:
 *   1) apply the change to the given UObject
 *   2) return a new FSwapChange that does the "opposite" action
 */
class FSwapChange : public FChange
{

public:
	virtual EChangeStyle GetChangeType() final
	{
		return FChange::EChangeStyle::InPlaceSwap;
	}

	/** Makes the change to the object */
	virtual void Apply(UObject* Object) final
	{
		check(false);
	}

	/** Reverts change to the object */
	virtual void Revert(UObject* Object) final
	{
		check(false);
	}

};


/**
 * To use FCommandChange you must implement Apply() and Revert()
 * Revert() is called to "Undo" and Apply() is called to "Redo"
 */
class FCommandChange : public FChange
{

public:
	virtual EChangeStyle GetChangeType() final
	{
		return FChange::EChangeStyle::CommandPattern;
	}

	virtual TUniquePtr<FChange> Execute(UObject* Object) final
	{
		check(false);
		return nullptr;
	}

};






struct FCompoundChangeInput
{
	FCompoundChangeInput()
	{
	}

	explicit FCompoundChangeInput( FCompoundChangeInput&& RHS )
		: Subchanges( MoveTemp( RHS.Subchanges ) )
	{
	}

	/** Ordered list of changes that comprise everything needed to describe this change */
	TArray<TUniquePtr<FChange>> Subchanges;

private:
	// Non-copyable
	FCompoundChangeInput( const FCompoundChangeInput& ) = delete;
	FCompoundChangeInput& operator=( const FCompoundChangeInput& ) = delete;
};



/**
 * FCompoundChange applies a sequence of FSwapChanges.
 * The changes are executed in reverse order (this is like a mini undo stack)
 */
class FCompoundChange : public FSwapChange
{

public:

	/** Constructor */
	explicit FCompoundChange( FCompoundChangeInput&& InitInput )
		: Input( MoveTemp( InitInput ) )
	{
	}

	// Parent class overrides
	CORE_API virtual TUniquePtr<FChange> Execute( UObject* Object ) override;
	CORE_API virtual FString ToString() const override;
	CORE_API virtual void PrintToLog( class FFeedbackContext& FeedbackContext, const int32 IndentLevel = 0 ) override;


private:
	
	/** The data we need to make this change */
	FCompoundChangeInput Input;

private:
	// Non-copyable
	FCompoundChange( const FCompoundChange& ) = delete;
	FCompoundChange& operator=( const FCompoundChange& ) = delete;

};


