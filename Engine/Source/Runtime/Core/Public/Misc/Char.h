// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Traits/IntType.h"
#include <ctype.h>
#include <wctype.h>
#include <type_traits>

/*-----------------------------------------------------------------------------
	Character type functions.
-----------------------------------------------------------------------------*/

/**
 * Templated literal struct to allow selection of wide or ansi string literals
 * based on the character type provided, and not on compiler switches.
 */
template <typename T> struct TLiteral
{
	static const ANSICHAR  Select(const ANSICHAR  ansi, const WIDECHAR ) { return ansi; }
	static const ANSICHAR* Select(const ANSICHAR* ansi, const WIDECHAR*) { return ansi; }
};

template <> struct TLiteral<WIDECHAR>
{
	static const WIDECHAR  Select(const ANSICHAR,  const WIDECHAR  wide) { return wide; }
	static const WIDECHAR* Select(const ANSICHAR*, const WIDECHAR* wide) { return wide; }
};

#define LITERAL(CharType, StringLiteral) TLiteral<CharType>::Select(StringLiteral, TEXT(StringLiteral))


template <typename CharType, const unsigned int Size>
struct TCharBase
{
	static constexpr CharType LineFeed           = (CharType)0xa;
	static constexpr CharType VerticalTab        = (CharType)0xb;
	static constexpr CharType FormFeed           = (CharType)0xc;
	static constexpr CharType CarriageReturn     = (CharType)0xd;
	static constexpr CharType NextLine           = (CharType)0x85;
	static constexpr CharType LineSeparator      = (CharType)0x2028;
	static constexpr CharType ParagraphSeparator = (CharType)0x2029;

	static bool IsLinebreak(CharType Char)
	{
		return ((uint32(Char) - LineFeed) <= uint32(CarriageReturn - LineFeed)) |
			(Char == NextLine) | (Char == LineSeparator) | (Char == ParagraphSeparator);
	}

};

template <typename CharType>
struct TCharBase<CharType, 1>
{
	static constexpr CharType LineFeed       = (CharType)0xa;
	static constexpr CharType VerticalTab    = (CharType)0xb;
	static constexpr CharType FormFeed       = (CharType)0xc;
	static constexpr CharType CarriageReturn = (CharType)0xd;

	static bool IsLinebreak(CharType Char)
	{
		return ((uint32(Char) - LineFeed) <= uint32(CarriageReturn - LineFeed));
	}
};


/**
 * TChar
 * Set of utility functions operating on a single character. The functions
 * are specialized for ANSICHAR and TCHAR character types. You can use the
 * typedefs FChar and FCharAnsi for convenience.
 */
template <typename CharType>
struct TChar : TCharBase<CharType, sizeof(CharType)>
{
	/**
	* Only converts ASCII characters, same as CRT to[w]upper() with standard C locale
	*/
	static CharType ToUpper(CharType Char)
	{
		return (CharType)(ToUnsigned(Char) - ((uint32(Char) - 'a' < 26u) << 5));
	}

	/**
	* Only converts ASCII characters, same as CRT to[w]upper() with standard C locale
	*/
	static CharType ToLower(CharType Char)
	{
		return (CharType)(ToUnsigned(Char) + ((uint32(Char) - 'A' < 26u) << 5));
	}

	static bool IsUpper(CharType Char)
	{
		if constexpr (std::is_same_v<CharType, ANSICHAR>)
		{
			return ::isupper((unsigned char)Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, WIDECHAR>)
		{
			return ::iswupper(Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, UTF8CHAR>)
		{
			return ::isupper((unsigned char)Char) != 0;
		}
		else
		{
			static_assert(sizeof(CharType) == 0, "Not supported");
			return false;
		}
	}

	static bool IsLower(CharType Char)
	{
		if constexpr (std::is_same_v<CharType, ANSICHAR>)
		{
			return ::islower((unsigned char)Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, WIDECHAR>)
		{
			return ::iswlower(Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, UTF8CHAR>)
		{
			return ::islower((unsigned char)Char) != 0;
		}
		else
		{
			static_assert(sizeof(CharType) == 0, "Not supported");
			return false;
		}
	}

	static bool IsAlpha(CharType Char)
	{
		if constexpr (std::is_same_v<CharType, ANSICHAR>)
		{
			return ::isalpha((unsigned char)Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, WIDECHAR>)
		{
			return ::iswalpha(Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, UTF8CHAR>)
		{
			return ::isalpha((unsigned char)Char) != 0;
		}
		else
		{
			static_assert(sizeof(CharType) == 0, "Not supported");
			return false;
		}
	}

	static bool IsGraph(CharType Char)
	{
		if constexpr (std::is_same_v<CharType, ANSICHAR>)
		{
			return ::isgraph((unsigned char)Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, WIDECHAR>)
		{
			return ::iswgraph(Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, UTF8CHAR>)
		{
			return ::isgraph((unsigned char)Char) != 0;
		}
		else
		{
			static_assert(sizeof(CharType) == 0, "Not supported");
			return false;
		}
	}

	static bool IsPrint(CharType Char)
	{
		if constexpr (std::is_same_v<CharType, ANSICHAR>)
		{
			return ::isprint((unsigned char)Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, WIDECHAR>)
		{
			return ::iswprint(Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, UTF8CHAR>)
		{
			return ::isprint((unsigned char)Char) != 0;
		}
		else
		{
			static_assert(sizeof(CharType) == 0, "Not supported");
			return false;
		}
	}

	static bool IsPunct(CharType Char)
	{
		if constexpr (std::is_same_v<CharType, ANSICHAR>)
		{
			return ::ispunct((unsigned char)Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, WIDECHAR>)
		{
			return ::iswpunct(Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, UTF8CHAR>)
		{
			return ::ispunct((unsigned char)Char) != 0;
		}
		else
		{
			static_assert(sizeof(CharType) == 0, "Not supported");
			return false;
		}
	}

	static bool IsAlnum(CharType Char)
	{
		if constexpr (std::is_same_v<CharType, ANSICHAR>)
		{
			return ::isalnum((unsigned char)Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, WIDECHAR>)
		{
			return ::iswalnum(Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, UTF8CHAR>)
		{
			return ::isalnum((unsigned char)Char) != 0;
		}
		else
		{
			static_assert(sizeof(CharType) == 0, "Not supported");
			return false;
		}
	}

	static bool IsDigit(CharType Char)
	{
		if constexpr (std::is_same_v<CharType, ANSICHAR>)
		{
			return ::isdigit((unsigned char)Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, WIDECHAR>)
		{
			return ::iswdigit(Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, UTF8CHAR>)
		{
			return ::isdigit((unsigned char)Char) != 0;
		}
		else
		{
			static_assert(sizeof(CharType) == 0, "Not supported");
			return false;
		}
	}

	static bool IsHexDigit(CharType Char)
	{
		if constexpr (std::is_same_v<CharType, ANSICHAR>)
		{
			return ::isxdigit((unsigned char)Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, WIDECHAR>)
		{
			return ::iswxdigit(Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, UTF8CHAR>)
		{
			return ::isxdigit((unsigned char)Char) != 0;
		}
		else
		{
			static_assert(sizeof(CharType) == 0, "Not supported");
			return false;
		}
	}

	static bool IsWhitespace(CharType Char)
	{
		if constexpr (std::is_same_v<CharType, ANSICHAR>)
		{
			return ::isspace((unsigned char)Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, WIDECHAR>)
		{
			return ::iswspace(Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, UTF8CHAR>)
		{
			return ::isspace((unsigned char)Char) != 0;
		}
		else
		{
			static_assert(sizeof(CharType) == 0, "Not supported");
			return false;
		}
	}

	static bool IsControl(CharType Char)
	{
		if constexpr (std::is_same_v<CharType, ANSICHAR>)
		{
			return ::iscntrl((unsigned char)Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, WIDECHAR>)
		{
			return ::iswcntrl(Char) != 0;
		}
		else if constexpr (std::is_same_v<CharType, UTF8CHAR>)
		{
			return ::iscntrl((unsigned char)Char) != 0;
		}
		else
		{
			static_assert(sizeof(CharType) == 0, "Not supported");
			return false;
		}
	}

	static bool IsOctDigit(CharType Char)
	{
		return uint32(Char) - '0' < 8u;
	}

	static int32 ConvertCharDigitToInt(CharType Char)
	{
		return static_cast<int32>(Char) - static_cast<int32>('0');
	}

	static bool IsIdentifier(CharType Char)
	{
		return IsAlnum(Char) || IsUnderscore(Char);
	}

	static bool IsUnderscore(CharType Char)
	{
		return Char == LITERAL(CharType, '_');
	}

	/**
	* Avoid sign extension problems with signed characters smaller than int
	*
	* E.g. 'Ö' - 'A' is negative since the char 'Ö' (0xD6) is negative and gets
	* sign-extended to the 32-bit int 0xFFFFFFD6 before subtraction happens.
	*
	* Mainly needed for subtraction and addition.
	*/
	static constexpr UE_FORCEINLINE_HINT uint32 ToUnsigned(CharType Char)
	{
		return (typename TUnsignedIntType<sizeof(CharType)>::Type)Char;
	}
};

typedef TChar<TCHAR>    FChar;
typedef TChar<WIDECHAR> FCharWide;
typedef TChar<ANSICHAR> FCharAnsi;
