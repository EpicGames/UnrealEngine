// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"

/*-----------------------------------------------------------------------------
	Command line.
-----------------------------------------------------------------------------*/

#ifndef UE_COMMAND_LINE_USES_ALLOW_LIST
	#ifdef WANTS_COMMANDLINE_WHITELIST
		#pragma message("Use UE_COMMAND_LINE_USES_ALLOW_LIST instead of WANTS_COMMANDLINE_WHITELIST")
		#define UE_COMMAND_LINE_USES_ALLOW_LIST WANTS_COMMANDLINE_WHITELIST
	#else
		#define UE_COMMAND_LINE_USES_ALLOW_LIST 0
	#endif
#endif


enum class ECommandLineArgumentFlags
{
	None = 0x0000,

	//~ Application context for a commandline argument
	EditorContext = 0x0001,
	ClientContext = 0x0002,
	ServerContext = 0x0004,
	CommandletContext = 0x0008,
	ProgramContext = 0x0010,

	GameContexts = 0x0006,
	AllContexts = 0x001F,

	// Indicates this argument should be automatically inherited by subprocesses.
	// To be combined with one or more application context to inherit to.
	Inherit = 0x0020,
};

ENUM_CLASS_FLAGS(ECommandLineArgumentFlags)

struct FCommandLine
{
	/** maximum size of the command line */
	static constexpr inline uint32 MaxCommandLineSize = 16384;

	/** 
	 * Resets FCommandLine to an uninitialised state as if Set() has never been called
	 */
	CORE_API static void Reset();

	/** 
	 * Returns an edited version of the executable's command line with the game name and certain other parameters removed. 
	 */
	CORE_API static const TCHAR* Get();
	
	/**
	 * Returns an edited version of the executable's command line. 
	 */
	CORE_API static const TCHAR* GetForLogging();

	/**
	 * Returns the command line originally passed to the executable.
	 */
	CORE_API static const TCHAR* GetOriginal();

	/**
	 * Returns an edited version of the command line originally passed to the executable.
	 */
	CORE_API static const TCHAR* GetOriginalForLogging();

	/** 
	 * Checks if the command line has been initialized. 
	 */
	CORE_API static bool IsInitialized();

	/**
	 * Gets a number representing this version of the command line, incremented each command line change
	 */
	CORE_API static uint32 GetCommandLineVersion();

	/**
	 * Sets CmdLine to the string given
	 */
	CORE_API static bool Set(const TCHAR* NewCommandLine);

	/**
	 * Appends the passed string to the command line as it is (no space is added).
	 * @param AppendString String that should be appended to the commandline.
	 */
	CORE_API static void Append(const TCHAR* AppendString);

	/**
	* Registers a command line argument and associates it with a set of flags and optionally a description.
	*
	* @param Name 				Argument name on the commandline that should be inherited.
	* @param Flags 				Flags to be associated with the argument.
	*/
	CORE_API static void RegisterArgument(FStringView Name, ECommandLineArgumentFlags Flags, FStringView Description = FStringView());

	/**
	 * Adds a new parameter to subprocess command line. If Param
	 * does not start with a space, one is added.
	 *
	 * @param Param Command line param string.
	 * @param ApplicationContextFlags The one or more application context flags to which this parameter applies.
	 */
	CORE_API static void AddToSubprocessCommandLine( const TCHAR* Param, ECommandLineArgumentFlags ApplicationContextFlags );

	/**
	* Builds a commandline of inheritable and subprocess arguments for a specified application context.
	*
	* @param Flags 				The flags indicating the subprocess context we want a commandline for.
	* @param bOnlyInherited 	If true, only add inherited arguments, not explicitly supplied subprocess arguments.
	* @param OutCommandline 	[out] The composed commandline.
	*/
	CORE_API static void BuildSubprocessCommandLine(ECommandLineArgumentFlags ApplicationContextFlags, bool bOnlyInherited, FStringBuilderBase& OutCommandline);

	/** 
	* Removes the executable name from the passed CmdLine, denoted by parentheses.
	* Returns the CmdLine string without the executable name.
	*/
	CORE_API static const TCHAR* RemoveExeName(const TCHAR* CmdLine);

	/**
	 * Parses a string into tokens, separating switches (beginning with -) from
	 * other parameters
	 *
	 * @param	CmdLine		the string to parse
	 * @param	Tokens		[out] filled with all parameters found in the string
	 * @param	Switches	[out] filled with all switches found in the string
	 */
	CORE_API static void Parse(const TCHAR* CmdLine, TArray<FString>& Tokens, TArray<FString>& Switches);

	/**
	 * Checks if command line logging filtering is enabled
	 *
	 * Returns true if logging filter is enabled
	 */
	CORE_API static bool IsCommandLineLoggingFiltered();

	/**
	 * Builds a command line string from the ArgC/ArgV main() arguments, together with
	 * an optional prefix or suffix, for adding additional command list arguments programmatically.
	*/
	CORE_API static FString BuildFromArgV(const WIDECHAR* Prefix, int32 ArgC, WIDECHAR* ArgV[], const WIDECHAR* Suffix);
	CORE_API static FString BuildFromArgV(const ANSICHAR* Prefix, int32 ArgC, ANSICHAR* ArgV[], const ANSICHAR* Suffix);

	/**
	* FilterCLIUsingGrammarBasedParser parses CLI style arguments.
	* filters for commands or keys specified in the AllowedList
	* and writes the to the OutLine.
	* OutLine and InLine may be point to the same buffer.
	* 
	* @param OutLine [out] the destination of the filtered InLine
	* @param MaxLen  the maximum length the OutLIne can hold.
	* @param InLine  the CLI to be filtered.
	* @param AllowedList  the list of commands or key permitted to pass the filter.
	* 
	* Returns true if Outline was large enough to hold the filtered string, Otherwise returns false.
	*/
	static bool FilterCLIUsingGrammarBasedParser(TCHAR* OutLine, int32 MaxLen, const TCHAR* InLine, const TArrayView<FString>& AllowedList);
private:
#if UE_COMMAND_LINE_USES_ALLOW_LIST
	/** Filters both the original and current command line list for approved only args */
	static void ApplyCommandLineAllowList();
	/** Filters any command line args that aren't on the approved list */
	static TArray<FString> FilterCommandLine(TCHAR* CommandLine);
	/** Filters any command line args that are on the to-strip list */
	static TArray<FString> FilterCommandLineForLogging(TCHAR* CommandLine);
	/** Rebuilds the command line using the filtered args */
	static void BuildCommandLineAllowList(TCHAR* CommandLine, uint32 Length, const TArray<FString>& FilteredArgs);
	static TArray<FString> ApprovedArgs;
	static TArray<FString> FilterArgsForLogging;
#else
	#define ApplyCommandLineAllowList()
#endif

	/** Flag to check if the commandline has been initialized or not. */
	static bool bIsInitialized;
	/** character buffer containing the command line */
	static TCHAR CmdLine[MaxCommandLineSize];
	/** character buffer containing the original command line */
	static TCHAR OriginalCmdLine[MaxCommandLineSize];
	/** character buffer containing the command line filtered for logging purposes */
	static TCHAR LoggingCmdLine[MaxCommandLineSize];
	/** character buffer containing the original command line filtered for logging purposes */
	static TCHAR LoggingOriginalCmdLine[MaxCommandLineSize];
	/** subprocess command lines */
	static FString& GetSubprocessCommandLine_Internal(ECommandLineArgumentFlags ContextFlags);
	/** add subprocess command lines */
	static void AddToSubprocessCommandLine_Internal(const TCHAR* Param, ECommandLineArgumentFlags ApplicationContextFlags);
	/** What version of the command line this is, incremented on any modification to the command line */
	static uint32 CmdLineVersion;

	struct FRegisteredArgData
	{
		ECommandLineArgumentFlags Flags = ECommandLineArgumentFlags::None;
		FString Description;
	};
	static TMap<FString, FRegisteredArgData> RegisteredArgs;
};

