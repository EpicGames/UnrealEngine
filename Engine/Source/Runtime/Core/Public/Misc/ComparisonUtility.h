// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "HAL/Platform.h"
#include "UObject/NameTypes.h"

namespace UE::ComparisonUtility {

/** Compare the two names, correctly ordering any numeric suffixes they may have */
CORE_API int32 CompareWithNumericSuffix(FName A, FName B);

/** Compare the two strings, correctly ordering any numeric suffixes they may have */
CORE_API int32 CompareWithNumericSuffix(FStringView A, FStringView B);

/** Compare the two strings, correctly ordering any numeric components they may have */
CORE_API int32 CompareNaturalOrder(FStringView A, FStringView B);

} // namespace UE::ComparisonUtility
