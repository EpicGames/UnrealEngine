// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Enumerates possible results of a compilation operation.
 *
 * This enum has to be compatible with the one defined in the
 * Engine\Source\Programs\UnrealBuildTool\System\ExternalExecution.cs file
 * to keep communication between UHT, UBT and Editor compiling processes valid.
 */
namespace ECompilationResult
{
	enum Type
	{
		/** Compilation succeeded */
		Succeeded = 0,
		/** Build was canceled, this is used on the engine side only */
		Canceled = 1,
		/** All targets were up to date, used only with -canskiplink */
		UpToDate = 2,
		/** The process has most likely crashed. This is what UE returns in case of an assert */
		CrashOrAssert = 3,
		/** Compilation failed because generated code changed which was not supported */
		FailedDueToHeaderChange = 4,
		/** Compilation failed due to the engine modules needing to be rebuilt */
		FailedDueToEngineChange = 5,
		/** Compilation failed due to compilation errors */
		OtherCompilationError = 6,
		/** Compilation failed due to live coding limit reached */
		LiveCodingLimitError = 7,
		/** Compilation failed due to TargetRules or ModuleRules errors */
		RulesError = 8,
		/** Compilation failed due to invalid action graph */
		ActionGraphInvalid = 9,
		/** Another instance of UBT was already running */
		ConflictingInstance = 10,
		/** Compilation is not supported in the current build */
		Unsupported,
		/** Unknown error */
		Unknown 
	};

	/**
	* Converts ECompilationResult enum to string.
	*/
	inline const TCHAR* ToString(ECompilationResult::Type Result)
	{
		switch (Result)
		{
		case ECompilationResult::UpToDate:
			return TEXT("UpToDate");
		case ECompilationResult::Canceled:
			return TEXT("Canceled");
		case ECompilationResult::Succeeded:
			return TEXT("Succeeded");
		case ECompilationResult::FailedDueToHeaderChange:
			return TEXT("FailedDueToHeaderChange");
		case ECompilationResult::OtherCompilationError:
			return TEXT("OtherCompilationError");
		case ECompilationResult::CrashOrAssert:
			return TEXT("CrashOrAssert");
		case ECompilationResult::LiveCodingLimitError:
			return TEXT("LiveCodingLimitError");
		case ECompilationResult::RulesError:
			return TEXT("RulesError");
		case ECompilationResult::ActionGraphInvalid:
			return TEXT("ActionGraphInvalid");
		case ECompilationResult::Unsupported:
			return TEXT("Unsupported");
		};
		return TEXT("Unknown");
	}

	/**
	* Returns false if the provided Result is either UpToDate or Succeeded.
	*/
	inline bool Failed(ECompilationResult::Type Result)
	{
		return !(Result == ECompilationResult::Succeeded || Result == ECompilationResult::UpToDate);
	}
}
