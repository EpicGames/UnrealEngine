// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "UObject/NameTypes.h"

/**
 * Growable compressed buffer. Usage is to append frequently but only request and therefore decompress
 * very infrequently. The prime usage case is the memory profiler keeping track of full call stacks.
 */
struct FCompressedGrowableBuffer
{
public:
	/**
	 * Constructor
	 *
	 * @param	MaxPendingBufferSize	Max chunk size to compress in uncompressed bytes
	 * @param	CompressionFlags		Compression flags to compress memory with
	 */
	CORE_API FCompressedGrowableBuffer(int32 MaxPendingBufferSize, FName CompressionFormat, ECompressionFlags CompressionFlags=COMPRESS_None);

	/**
	 * Locks the buffer for reading. Needs to be called before calls to Access and needs
	 * to be matched up with Unlock call.
	 */
	CORE_API void Lock();
	/**
	 * Unlocks the buffer and frees temporary resources used for accessing.
	 */
	CORE_API void Unlock();

	/**
	 * Appends passed in data to the buffer. The data needs to be less than the max
	 * pending buffer size. The code will assert on this assumption.	 
	 *
	 * @param	Data	Data to append
	 * @param	Size	Size of data in bytes.
	 * @return	Offset of data, used for retrieval later on
	 */
	CORE_API int32 Append( void* Data, int32 Size );

	/**
	 * Accesses the data at passed in offset and returns it. The memory is read-only and
	 * memory will be freed in call to unlock. The lifetime of the data is till the next
	 * call to Unlock, Append or Access
	 *
	 * @param	Offset	Offset to return corresponding data for
	 */
	CORE_API void* Access( int32 Offset );

	/**
	 * @return	Number of entries appended.
	 */
	int32 Num() const
	{
		return NumEntries;
	}

	/** 
	 * Helper function to return the amount of memory allocated by this buffer 
	 *
	 * @return number of bytes allocated by this buffer
	 */
	SIZE_T GetAllocatedSize() const
	{
		return CompressedBuffer.GetAllocatedSize()
			+ PendingCompressionBuffer.GetAllocatedSize()
			+ DecompressedBuffer.GetAllocatedSize()
			+ BookKeepingInfo.GetAllocatedSize();
	}

private:

	/** Helper structure for book keeping. */
	struct FBufferBookKeeping
	{
		/** Offset into compressed data.				*/
		int32 CompressedOffset;
		/** Size of compressed data in this chunk.		*/
		int32 CompressedSize;
		/** Offset into uncompressed data.				*/
		int32 UncompressedOffset;
		/** Size of uncompressed data in this chunk.	*/
		int32 UncompressedSize;
	};

	/** Maximum chunk size to compress in uncompressed bytes.				*/
	int32					MaxPendingBufferSize;
	/** Compression format used to compress the data.						*/
	FName				CompressionFormat;
	/** Compression flags used to compress the data.						*/
	ECompressionFlags	CompressionFlags;
	/** Current offset in uncompressed data.								*/
	int32					CurrentOffset;
	/** Number of entries in buffer.										*/
	int32					NumEntries;
	/** Compressed data.													*/
	TArray<uint8>		CompressedBuffer;
	/** Data pending compression once size limit is reached.				*/
	TArray<uint8>		PendingCompressionBuffer;
	/** Temporary decompression buffer used between Lock/ Unlock.			*/
	TArray<uint8>		DecompressedBuffer;
	/** Index into book keeping info associated with decompressed buffer.	*/
	int32					DecompressedBufferBookKeepingInfoIndex;
	/** Book keeping information for decompression/ access.					*/
	TArray<FBufferBookKeeping>	BookKeepingInfo;
};
