// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "Misc/CompressionFlags.h"
#include "Templates/Atomic.h"
#include "UObject/NameTypes.h"

class IMemoryReadStream;
template <typename T> class TAtomic;

// Define global current platform default to current platform.  
// DEPRECATED, USE NAME_Zlib - was deprecated in 2018, leaving for grep until 5.6 or so.
//#define COMPRESS_Default			COMPRESS_ZLIB

/**
 * Chunk size serialization code splits data into. The loading value CANNOT be changed without resaving all
 * compressed data which is why they are split into two separate defines.
 */
#define LOADING_COMPRESSION_CHUNK_SIZE_PRE_369  32768
#define LOADING_COMPRESSION_CHUNK_SIZE			131072
#define SAVING_COMPRESSION_CHUNK_SIZE			LOADING_COMPRESSION_CHUNK_SIZE

struct FCompression
{
	/**
	* CompressionData is per-compressor data passed through this API and is interpreted by the codecs.
	* For builtins:
	*	NAME_Zlib	BitWindow
	*	NAME_LZ4	Unused
	*	NAME_Gzip	Unused
	*	NAME_Oodle	Unused
	*/

	CORE_API static void Init(void);

	/** Time spent compressing data in cycles. */
	CORE_API static TAtomic<uint64> CompressorTimeCycles;
	/** Number of bytes before compression.		*/
	CORE_API static TAtomic<uint64> CompressorSrcBytes;
	/** Number of bytes after compression.		*/
	CORE_API static TAtomic<uint64> CompressorDstBytes;


	/**
	 * Returns a version number for a specified format
	 *
	 * @param	FormatName					Compressor format name (eg NAME_Zlib)
	 * @return								An interpretation of an internal version number for a specific format (different formats will have different layouts) this should change if a version is updated
	 */
	CORE_API static uint32 GetCompressorVersion(FName FormatName);

	/**
	 * Thread-safe abstract compression routine to query buffer requirements for a compression operation.
	 * This is the minimize size to allocate the buffer for CompressMemory (encoding).
	 * Use GetMaximumCompressedSize at decode to know how large a compressed buffer may be.
	 *
	 * Despite the name, this has nothing to do with how much memory will be allocated during compression.
	 * 
	 * @param	FormatName					Name of the compression format
	 * @param	UncompressedSize			Size of uncompressed data in bytes
	 * @param	CompressionData				Additional compression parameter - see comments a top of FCompression struct.
	 * @param	OutBufferSizeRequired		The maximum possible bytes needed for compression of a data buffer of the given size.
	 * @return  false if the bounds couldn't be determined - e.g. the format is unknown or the size isn't supported by the format.
	*/
	CORE_API static bool CompressMemoryBound(FName InFormatName, int64& OutBufferSizeRequired, int64 InUncompressedSizeBytes, uintptr_t InCompressionData = 0);

	// Thunks to the 64 bit, prefer that as this can only Fatal on error!
	CORE_API static int32 CompressMemoryBound(FName FormatName, int32 UncompressedSize, ECompressionFlags Flags = COMPRESS_NoFlags, int32 CompressionData = 0);
	
	/**
	 * Thread-safe abstract compression routine to query maximum compressed size that could be made. This is used for making buffers that could hold
	 * any compressed buffer that started uncompressed with the given size. It's distinct from the buffer needed to compress! If you are calling CompressMemory then
	 * you need to pass a buffer sized based on CompressMemoryBound!
	 * 
	 * CompressMemoryBound is strictly greater equal GetMaximumCompressedSize.
	 *
	 * @param	FormatName					Name of the compression format
	 * @param	OutMaxCompressedSize		The maximum possible size of valid compressed data made by this format
	 * @param	InUncompressedSizeBytes		Size of uncompressed data in bytes	 
	 * @param	CompressionData				Additional compression parameter - see comments a top of FCompression struct.
	 * @return success
	 */
	CORE_API static bool GetMaximumCompressedSize(FName InFormatName, int64& OutMaxCompressedSize, int64 InUncompressedSizeBytes, uintptr_t InCompressionData=0);

	/**
	 * Thread-safe abstract compression routine. Compresses memory from uncompressed buffer and writes it to compressed
	 * buffer. Updates CompressedSize with size of compressed data. Compression controlled by the passed in flags.
	 * CompressMemory is expected to return true and write valid data even if it expanded bytes.
	 * Always check CompressedSize >= UncompressedSize and fall back to uncompressed, or use CompressMemoryIfWorthDecompressing
	 *
	 * @param	FormatName					Name of the compression format
	 * @param	CompressedBuffer			Buffer compressed data is going to be written to
	 * @param	CompressedSize	[in/out]	Size of CompressedBuffer, at exit will be size of compressed data
	 * @param	UncompressedBuffer			Buffer containing uncompressed data
	 * @param	UncompressedSize			Size of uncompressed data in bytes
	 * @param	Flags						Flags to control what method to use and optionally control memory vs speed
	 * @param	CompressionData				Additional compression parameter - see comments a top of FCompression struct.
	 * @return true if compression succeeds, false if it fails because CompressedBuffer was too small or other reasons
	 */
	CORE_API static bool CompressMemory(FName FormatName, void* CompressedBuffer, int64& CompressedSize, const void* UncompressedBuffer, int64 UncompressedSize, ECompressionFlags Flags = COMPRESS_NoFlags, uintptr_t CompressionData = 0);

	// Thunks to 64 bit CompressMemory, favor that.
	CORE_API static bool CompressMemory(FName FormatName, void* CompressedBuffer, int32& CompressedSize, const void* UncompressedBuffer, int32 UncompressedSize, ECompressionFlags Flags = COMPRESS_NoFlags, int32 CompressionData = 0);
	
	/**
	* Same as CompressMemory but evaluates if the compression gain is worth the runtime decode time.
	 *
	 * @param	FormatName					Name of the compression format
	 * @param	bOutWasCompressed [out]		Whether the compression was worth it. The data is still compressed.
	 * @param	MinBytesSaved				Minimum amount of bytes which should be saved when performing compression, otherwise false is returned
	 * @param	MinPercentSaved				Minimum percentage of the buffer which should be saved when performing compression, otherwise false is returned
	 * @param	CompressedBuffer			Buffer compressed data is going to be written to
	 * @param	CompressedSize	[in/out]	Size of CompressedBuffer, at exit will be size of compressed data
	 * @param	UncompressedBuffer			Buffer containing uncompressed data
	 * @param	UncompressedSize			Size of uncompressed data in bytes
	 * @param	Flags						Flags to control what method to use and optionally control memory vs speed
	 * @param	CompressionData				Additional compression parameter - see comments a top of FCompression struct.
	 * @return true if compression succeeds, false if it fails because CompressedBuffer was too small or other reasons
	 */
	CORE_API static bool CompressMemoryIfWorthDecompressing(FName FormatName, bool& bOutWasCompressed, int64 MinBytesSaved, int32 MinPercentSaved, void* CompressedBuffer, int64& CompressedSize, const void* UncompressedBuffer, int64 UncompressedSize, ECompressionFlags Flags = COMPRESS_NoFlags, uintptr_t CompressionData = 0);

	// 32 bit thunk. This legacy function doens't separate bOutWasCompressed vs general compress failure and will return false in both cases.
	CORE_API static bool CompressMemoryIfWorthDecompressing(FName FormatName, int32 MinBytesSaved, int32 MinPercentSaved, void* CompressedBuffer, int32& CompressedSize, const void* UncompressedBuffer, int32 UncompressedSize, ECompressionFlags Flags = COMPRESS_NoFlags, int32 CompressionData = 0);

	/**
	 * Thread-safe abstract decompression routine. Uncompresses memory from compressed buffer and writes it to uncompressed
	 * buffer. UncompressedSize is expected to be the exact size of the data after decompression.
	 *
	 * @param	FormatName					Name of the compression format
	 * @param	UncompressedBuffer			Buffer containing uncompressed data
	 * @param	UncompressedSize			Size of uncompressed data in bytes
	 * @param	CompressedBuffer			Buffer compressed data is going to be read from
	 * @param	CompressedSize				Size of CompressedBuffer data in bytes
	 * @param	Flags						Flags to control what method to use to decompress
	 * @param	CompressionData				Additional decompression parameter (specifies BitWindow value for ZLIB compression format)
	 * @return true if compression succeeds, false if it fails because CompressedBuffer was too small or other reasons
	 */
	CORE_API static bool UncompressMemory(FName FormatName, void* UncompressedBuffer, int64 UncompressedSize, const void* CompressedBuffer, int64 CompressedSize, ECompressionFlags Flags=COMPRESS_NoFlags, uintptr_t CompressionData=0);

	/**
	* Wraps UncompressMemory to provide access with a IMemoryReadStream interface, attempting to avoid copying in to a contiguous buffer doe decompression
	* if the format supports it. **If not, a CompressedSize temp buffer will be allocated.**
	*/
	CORE_API static bool UncompressMemoryStream(FName FormatName, void* UncompressedBuffer, int64 UncompressedSize, IMemoryReadStream* Stream, int64 StreamOffset, int64 CompressedSize, ECompressionFlags Flags = COMPRESS_NoFlags, uintptr_t CompressionData = 0);


	/**
	 * Returns a string which can be used to identify if a format has become out of date
	 *
	 * @param	FormatName					name of the format to retrieve the DDC suffix for
	 * @return	unique DDC key string which will be different when the format is changed / updated
	 */
	CORE_API static FString GetCompressorDDCSuffix(FName FormatName);

	/**
	 * Checks to see if a format will be usable, so that a fallback can be used
	 * @param FormatName The name of the format to test
	 */
	CORE_API static bool IsFormatValid(FName FormatName);

	/**
	 * Verifies if the passed in value represents valid compression flags
	 * @param InCompressionFlags Value to test
	 */
	CORE_API static bool VerifyCompressionFlagsValid(int32 InCompressionFlags);

	// deprecating this because we "should", but all internal references to these flags were deprecated in 4.21 so hopefully no one is using them anymore!
	UE_DEPRECATED(5.5, "Use the direct FName instead e.g NAME_Oodle")
	CORE_API static FName GetCompressionFormatFromDeprecatedFlags(ECompressionFlags DeprecatedFlags);

private:
	
	/**
	 * Find a compression format module by name, returning nullptr if no module found
	 */
	static struct ICompressionFormat* GetCompressionFormat(FName Method, bool bErrorOnFailure=true);

	/** Mapping of Compression FNames to their compressor objects */
	static TMap<FName, struct ICompressionFormat*> CompressionFormats;
	static FCriticalSection CompressionFormatsCriticalSection;

};


