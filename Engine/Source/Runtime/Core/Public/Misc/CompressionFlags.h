// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** Default compressor bit window for Zlib */
#define DEFAULT_ZLIB_BIT_WINDOW		15

/**
* Flags controlling [de]compression
* Make sure to update VerifyCompressionFlagsValid after changing these values.
*/
enum ECompressionFlags
{
	/** No flags																*/
	COMPRESS_None = 0x00,

	/** Compress with ZLIB - DEPRECATED, USE FNAME									*/
	COMPRESS_ZLIB_DEPRECATED = 0x01,
	/** Compress with GZIP - DEPRECATED, USE FNAME									*/
	COMPRESS_GZIP_DEPRECATED = 0x02,
	/** Compress with user defined callbacks - DEPRECATED, USE FNAME                */
	COMPRESS_Custom_DEPRECATED = 0x04,
	/** Joint of the previous ones to determine if old flags are being used			*/
	COMPRESS_DeprecatedFormatFlagsMask = 0xF,


	/** No flags specified /														*/
	COMPRESS_NoFlags = 0x00,
	/** Prefer compression that compresses smaller (ONLY VALID FOR COMPRESSION)		*/
	COMPRESS_BiasMemory = 0x10,
	COMPRESS_BiasSize = COMPRESS_BiasMemory,
	/** Prefer compression that compresses faster (ONLY VALID FOR COMPRESSION)		*/
	COMPRESS_BiasSpeed = 0x20,
	/** Is the source buffer padded out	(ONLY VALID FOR UNCOMPRESS)					*/
	COMPRESS_SourceIsPadded = 0x80,

	/** Set of flags that are options are still allowed								*/
	COMPRESS_OptionsFlagsMask = 0xF0,

	/** Indicate this compress call is for Packaging (pak/iostore) */
	COMPRESS_ForPackaging = 0x100,
	COMPRESS_ForPurposeMask = 0xF00,
};

