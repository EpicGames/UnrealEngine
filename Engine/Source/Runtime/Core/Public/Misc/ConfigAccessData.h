// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ConfigAccessTracking.h"

#if UE_WITH_CONFIG_TRACKING || WITH_EDITOR
#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Interfaces/ITargetPlatform.h"
#include "Templates/TypeHash.h"

namespace UE::ConfigAccessTracking
{

/**
 * Full path of a read of an FConfigValue; used to persistently track which config values were read during
 * an invocation of the editor.
 */
struct FConfigAccessData
{
	FNameEntryId ConfigPlatform;
	FNameEntryId FileName;
	FNameEntryId SectionName;
	FMinimalName ValueName;
	const ITargetPlatform* RequestingPlatform = nullptr;
	ELoadType LoadType = ELoadType::Uninitialized;

	FConfigAccessData() = default;
	FConfigAccessData(ELoadType InLoadType, FName InConfigPlatform, FName InFileName,
		FName InSectionName, FName InValueName, const ITargetPlatform* InRequestingPlatform);
	CORE_API FConfigAccessData(ELoadType InLoadType, FNameEntryId InConfigPlatform, FNameEntryId InFileName,
		FNameEntryId InSectionName, FMinimalName InValueName, const ITargetPlatform* InRequestingPlatform);
	CORE_API FConfigAccessData(ELoadType InLoadType, FNameEntryId InConfigPlatform, FNameEntryId InFileName);

	FName GetConfigPlatform() const;
	FName GetFileName() const;
	FName GetSectionName() const;
	FName GetValueName() const;

	CORE_API FConfigAccessData GetFileOnlyData() const;
	CORE_API FConfigAccessData GetPathOnlyData() const;
	CORE_API FString FullPathToString() const;
	CORE_API void AppendFullPath(FStringBuilderBase& Out) const;

	/**
	 * "ConfigSystem.<Editor>.../../../Engine/Config/ConsoleVariables.ini:[Section]:Value"
	 *   -> "ConfigSystem", "<Editor>", "../../../Engine/Config/ConsoleVariables.ini", "Section", "Value"
	 */
	CORE_API static FConfigAccessData Parse(FStringView Text);

	friend uint32 GetTypeHash(const FConfigAccessData& Data);
	bool IsSameConfigFile(const FConfigAccessData& Other) const;
	bool operator==(const FConfigAccessData& Other) const;
	bool operator!=(const FConfigAccessData& Other) const;
	bool operator<(const FConfigAccessData& Other) const;
};

CORE_API void EscapeConfigTrackingTokenToString(FName Token, FStringBuilderBase& Result);
CORE_API void EscapeConfigTrackingTokenAppendString(FName Token, FStringBuilderBase& Result);
CORE_API bool TryTokenizeConfigTrackingString(FStringView Text, TArrayView<FStringBuilderBase*> OutTokens);

constexpr FStringView PlatformAgnosticName = TEXTVIEW("<Editor>");

}

/** Convert ELoadType -> text */
const TCHAR* LexToString(UE::ConfigAccessTracking::ELoadType LoadType);

/** Convert text -> ELoadType */
void LexFromString(UE::ConfigAccessTracking::ELoadType& OutLoadType, FStringView Text);

///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////

namespace UE::ConfigAccessTracking
{

inline FConfigAccessData::FConfigAccessData(ELoadType InLoadType, FName InConfigPlatform, FName InFileName,
	FName InSectionName, FName InValueName, const ITargetPlatform* InRequestingPlatform)
	: FConfigAccessData(InLoadType, InConfigPlatform.GetComparisonIndex(), InFileName.GetComparisonIndex(),
		InSectionName.GetComparisonIndex(), FMinimalName(InValueName), InRequestingPlatform)
{
}

inline FName FConfigAccessData::GetConfigPlatform() const
{
	return FName(ConfigPlatform, ConfigPlatform, NAME_NO_NUMBER_INTERNAL);
}

inline FName FConfigAccessData::GetFileName() const
{
	return FName(FileName, FileName, NAME_NO_NUMBER_INTERNAL);
}

inline FName FConfigAccessData::GetSectionName() const
{
	return FName(SectionName, SectionName, NAME_NO_NUMBER_INTERNAL);
}

inline FName FConfigAccessData::GetValueName() const
{
	return FName(ValueName);
}

inline uint32 GetTypeHash(const UE::ConfigAccessTracking::FConfigAccessData& Data)
{
	uint32 Hash = static_cast<uint32>(Data.LoadType);
	Hash = HashCombineFast(Hash, Data.ConfigPlatform.ToUnstableInt());
	Hash = HashCombineFast(Hash, Data.FileName.ToUnstableInt());
	Hash = HashCombineFast(Hash, Data.SectionName.ToUnstableInt());
	Hash = HashCombineFast(Hash, GetTypeHash(Data.ValueName));
	Hash = HashCombineFast(Hash, GetTypeHash(Data.RequestingPlatform));
	return Hash;
}

inline bool FConfigAccessData::IsSameConfigFile(const FConfigAccessData& Other) const
{
	return LoadType == Other.LoadType && ConfigPlatform == Other.ConfigPlatform && FileName == Other.FileName;
}

inline bool FConfigAccessData::operator==(const FConfigAccessData& Other) const
{
	return LoadType == Other.LoadType && ConfigPlatform == Other.ConfigPlatform &&
		FileName == Other.FileName && SectionName == Other.SectionName && ValueName == Other.ValueName &&
		RequestingPlatform == Other.RequestingPlatform;
}

inline bool FConfigAccessData::operator!=(const FConfigAccessData& Other) const
{
	return !(*this == Other);
}

inline bool FConfigAccessData::operator<(const FConfigAccessData& Other) const
{
	if (LoadType != Other.LoadType)
	{
		return static_cast<uint32>(LoadType) < static_cast<uint32>(Other.LoadType);
	}
	if (ConfigPlatform != Other.ConfigPlatform)
	{
		return ConfigPlatform.LexicalLess(Other.ConfigPlatform);
	}
	if (FileName != Other.FileName)
	{
		return FileName.LexicalLess(Other.FileName);
	}
	if (SectionName != Other.SectionName)
	{
		return SectionName.LexicalLess(Other.SectionName);
	}
	if (ValueName != Other.ValueName)
	{
		return FName(ValueName).LexicalLess(FName(Other.ValueName));
	}
	if (RequestingPlatform != Other.RequestingPlatform)
	{
		if (RequestingPlatform == nullptr)
		{
			return true;
		}
		if (Other.RequestingPlatform == nullptr)
		{
			return false;
		}
		return RequestingPlatform->PlatformName() < Other.RequestingPlatform->PlatformName();
	}
	return false;
}

} // namespace UE::ConfigAccessTracking

#endif // UE_WITH_CONFIG_TRACKING || WITH_EDITOR