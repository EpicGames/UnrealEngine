// Copyright Epic Games, Inc. All Rights Reserved.

/*---------------------------------------------------------------------------------------------------------------------
	Access tracking for Config cache. When compiled in, adds extra data to FConfigFile structures so that we can report
	File,Section,Value names whenever an FConfigValue is read. The data about any accessed ConfigValue is reported to
	callbacks registered via UE::ConfigAccessTracking::AddConfigValueReadCallback.
---------------------------------------------------------------------------------------------------------------------*/
#pragma once

#include "HAL/Platform.h"
#include "UObject/NameTypes.h"

#define UE_WITH_CONFIG_TRACKING (WITH_EDITOR || 0)

#if UE_WITH_CONFIG_TRACKING
#include <atomic>
#include "Containers/StringView.h"
#include "Templates/Function.h"
#include "Templates/RefCounting.h"

class FConfigCacheIni;
class FConfigFile;
class FConfigSection;
struct FConfigValue;
#endif

#if WITH_EDITOR || UE_WITH_CONFIG_TRACKING

namespace UE::ConfigAccessTracking
{

/**
 * The manner in which the FConfigFile was loaded, so that subscribers can record
 * how to reload it in another process and reread the values.
 */
enum class ELoadType : uint8
{
	ConfigSystem,
	LocalIniFile,
	LocalSingleIniFile,
	ExternalIniFile,
	ExternalSingleIniFile,
	Manual,
	SuppressReporting,
	Uninitialized,
};

} // namespace UE::ConfigAccessTracking

#endif // WITH_EDITOR || UE_WITH_CONFIG_TRACKING

#if UE_WITH_CONFIG_TRACKING

namespace UE::ConfigAccessTracking
{

/**
 * A refcounted struct of data about an FConfigFile, including a backpointer to the FConfigFile if it is still 
 * in memory. References to this data are held by all the FConfigSections in the FConfigFile so that they can lookup
 * data about their FConfigFile when their FConfigValues are accessed. Due to FConfigSection being movable out of
 * FConfigFiles, this struct might outlive its FConfigFile (but its backpointer will be set to null).
 */
struct FFile : public FRefCountedObject
{
	CORE_API FFile(const FConfigFile* InConfigFile);
	CORE_API FName GetFilenameToLoad() const;
	CORE_API FName GetPlatformName();
	CORE_API void SetAsLoadTypeConfigSystem(FConfigCacheIni& ConfigSystem, FConfigFile& ConfigFile);

public:
	const FConfigFile* ConfigFile = nullptr;
	FName OverrideFilenameToLoad;
private:
	FName PlatformName = NAME_None;
	FStringView SavedConfigFilePlatformName;
	bool bSavedHasConfigFile : 1; // = false;
	bool bSavedConfigFileHasPlatformName : 1; // = false;
	bool bPlatformNameInitialized : 1; // = false;
	bool bOverridePlatformName : 1; // = false;
	bool bSavedOverridePlatformName : 1; // = false;
};

/**
 * A refcounted struct of data about an FConfigSection, including a backpointer to the UE::ConfigAccessTracking::FFile
 * that holds data about the section's FConfigFile. It does not hold a backpointer to the FConfigSection itself because
 * FConfigSections are value types in a TMap owned by the FConfigFIle and their address frequently changes.
 * References to this data are held by all FConfigValues in the FConfigSection. Due to FConfigSection being movable out
 * of FConfigFiles, this struct might outlive the existence of its FConfigSection in the FConfigFile, or even outlive
 * the FConfigFile.
 */
struct FSection : public FRefCountedObject
{
	CORE_API FSection(FFile& InFileAccess, FStringView InSectionName);

	TRefCountPtr<FFile> FileAccess;
	FNameEntryId SectionName;
};

/** Function type that for subscribers to AddConfigValueReadCallback. */
using FConfigValueReadCallbackFunc = TFunction<void(UE::ConfigAccessTracking::FSection* Section,
	FMinimalName ValueName, const FConfigValue& ConfigValue)>;

/** Handle used to remove a subscriber from AddConfigValueReadCallback. */
struct FConfigValueReadCallbackId
{
	int32 Id = -1;
	bool IsValid() { return Id != -1; }
};

/** Subscribe to events sent whenever any FConfigValue is read. */
CORE_API FConfigValueReadCallbackId AddConfigValueReadCallback(FConfigValueReadCallbackFunc Callback);
/** Remove a subscriber that was added via FConfigValue. */
CORE_API void RemoveConfigValueReadCallback(FConfigValueReadCallbackId DelegateHandle);

/** Disables recording of ConfigValues read on the current thread while in scope. */
struct FIgnoreScope
{
public:
	CORE_API FIgnoreScope();
	CORE_API ~FIgnoreScope();

private:
	bool bPreviousIgnoreReads = false;
};

namespace Private
{

/**
 * Stores the number of existing AddConfigValueReadCallback subscribers. We use it to avoid the cost
 * of a function call when no subscribers are registered.
 */
extern CORE_API std::atomic<int32> ConfigValueReadCallbackQuantity;

CORE_API void OnConfigValueReadInternal(UE::ConfigAccessTracking::FSection* Section, FMinimalName ValueName,
	const FConfigValue& ConfigValue);

inline void OnConfigValueRead(UE::ConfigAccessTracking::FSection* Section, FMinimalName ValueName,
	const FConfigValue& ConfigValue)
{
	if (ConfigValueReadCallbackQuantity.load(std::memory_order_acquire) > 0)
	{
		OnConfigValueReadInternal(Section, ValueName, ConfigValue);
	}
}

} // namespace Private

} // namespace UE::ConfigAccessTracking

#else  // UE_WITH_CONFIG_TRACKING

namespace UE::ConfigAccessTracking
{

// Declare FSection even though it is undefined so that we can pass FSection* as an argument to FConfigSection and FConfigValue
// constructors rather than having to create separate constructors under #ifdef UE_WITH_CONFIG_TRACKING
struct FSection;

// Define FIgnoreScope even when !UE_WITH_CONFIG_TRACKING so we don't need to wrap its use in a macro.
struct FIgnoreScope
{
	FIgnoreScope(){}
	~FIgnoreScope()= default;
};

} // namespace UE::ConfigAccessTracking

#endif // else !UE_WITH_CONFIG_TRACKING

