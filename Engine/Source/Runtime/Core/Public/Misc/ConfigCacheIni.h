// Copyright Epic Games, Inc. All Rights Reserved.

/*-----------------------------------------------------------------------------
	Config cache.
-----------------------------------------------------------------------------*/

#pragma once

#include "Algo/Reverse.h"
#include "Async/TransactionallySafeSharedMutex.h"
#include "Async/SharedMutex.h"
#include "Async/TransactionallySafeSharedRecursiveMutex.h"
#include "Async/SharedRecursiveMutex.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/List.h"
#include "Containers/SparseArray.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextLocalizationResource.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Math/MathFwd.h"
#include "Math/Rotator.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "Misc/Build.h"
#include "Misc/ConfigAccessTracking.h"
#include "Misc/ConfigTypes.h"
#include "Misc/Paths.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Serialization/StructuredArchiveSlots.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

class FConfigCacheIni;
class FConfigFile;
class FOutputDevice;
class IConsoleVariable;
struct FColor;

CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogConfig, Log, All);

// Server builds should be tweakable even in Shipping
#define ALLOW_INI_OVERRIDE_FROM_COMMANDLINE			(UE_SERVER || !(UE_BUILD_SHIPPING))
#define CONFIG_CAN_SAVE_COMMENTS (WITH_EDITOR)


using FConfigSystemMutex = UE::FTransactionallySafeSharedRecursiveMutex;

///////////////////////////////////////////////////////////////////////////////
// Info about the deprecation of functions returning non-const FConfigSections:
//   In a future change, we will be tracking operations done to config files (via GConfig, etc) for improved saving and allowing for plugin unloading.
//   To prepare for this, we need to remove the ability for code to directly modify config sections because then we can't track them. So, functions
//   that return non-cont FConfigSections have been deprecated - continuing to use them may cause these directly-modified settings to not be saved correctly
//
// If you are receiving deprecation messages, you should update your code ASAP. The deprecation messages will tell you how to fix that line, but if you
// were counting on modifying a section directly, or you were iterating over an FConfigFile with a ranged-for iterator ("for (auto& Pair : File)") you will need to
// make some additional code changes:
//
// Modifying:
//    * Replace your direct modification with calls to SetString, SetBool, etc for non-array values
//    * Replace your direct modifications of array type values with AddToSection, AddUniqueToSection, RemoveKeyFromSection, RemoveFromSection
//    * Fully construct a local new FConfigSection and then add that fully into the FConfigFile with Add
//
// Iterating over key/value pairs:
//    * Replace FConfigSection::TIterator with FConfigSection::TConstIterator
//
// Iterating over sections in a file:
//    * Ranged-for will need to use a const FConfigFile, to force the compiler to use the iterator that returns a const FConfigSection:
//       * for (auto& Pair : AsConst(MyFile))
//    * FConfigFile::TIterator did not seem to be used, but if you did use it, you should replace it with the above ranged-for version
///////////////////////////////////////////////////////////////////////////////




///////////////////////////////////////////////////////////////////////////////
//
// This is the master list of known ini files that are used and processed 
// on all platforms (specifically for runtime/binary speedups. Other, editor-
// specific inis, or non-hierarchical ini files will still work with the 
// old system, but they won't have any optimizations applied
//
// These should be listed in the order of highest to lowest use, for optimization
//
///////////////////////////////////////////////////////////////////////////////

#define ENUMERATE_KNOWN_INI_FILES(op) \
	op(Engine) \
	op(Game) \
	op(Input) \
	op(DeviceProfiles) \
	op(GameUserSettings) \
	op(Scalability) \
	op(RuntimeOptions) \
	op(InstallBundle) \
	op(Hardware) \
	op(GameplayTags)


#define KNOWN_INI_ENUM(IniName) IniName,
enum class EKnownIniFile : uint8
{
	// make an enum entry for each known ini above
	ENUMERATE_KNOWN_INI_FILES(KNOWN_INI_ENUM)
	
	// convenient counter for the above list
	NumKnownFiles,
};

class FConfigContext;
#if UE_WITH_CONFIG_TRACKING
UE::ConfigAccessTracking::FSection* GetSectionAccess(const FConfigSection* InSection);
#endif

struct FConfigValue
{
public:
	enum class EValueType : uint8
	{
		// Foo=Bar
		Set,
		// .Foo=Bar
		ArrayAdd,
		// +Foo=Bar
		ArrayAddUnique,
		// -Foo=Bar
		Remove,
		// !Foo=ClearArray
		Clear,
		// ^Array=Empty, means that this will clear any existing entries from an array property, unlike
		// Clear which will leave the array propperty untouched (as nothing is in GConfig for the key) 
		InitializeToEmpty,
		// @Array=StructKey
		ArrayOfStructKey,
		// *Array=PerObjectConfigStructKey
		POCArrayOfStructKey,
        // Virtual type, meaning it is the final combined result of set operations
        Combined,
        // Virtual type, meaning it is the final combined result of array operations
        ArrayCombined,
	};
	
	
	FConfigValue()
		: FConfigValue(nullptr, NAME_None)
	{}

	FConfigValue(const FConfigSection* InSection, FName InValueName, EValueType Type = EValueType::Combined)
		: ValueType(Type)
#if UE_WITH_CONFIG_TRACKING
		, SectionAccess(GetSectionAccess(InSection))
		, ValueName(InValueName)
#endif
	{
	}

	FConfigValue(const TCHAR* InValue, EValueType Type=EValueType::Combined)
		: FConfigValue(nullptr, NAME_None, InValue, Type)
	{}

	FConfigValue(const FConfigSection* InSection, FName InValueName, const TCHAR* InValue, EValueType Type = EValueType::Combined)
		: SavedValue(InValue)
		, ValueType(Type)
#if UE_WITH_CONFIG_TRACKING
		, SectionAccess(GetSectionAccess(InSection))
		, ValueName(InValueName)
#endif
	{
		SavedValueHash = FTextLocalizationResource::HashString(SavedValue);
		bExpandOnDemand = NeedsToExpandValue();
	}

	FConfigValue(const FString& InValue, EValueType Type=EValueType::Combined)
		: FConfigValue(nullptr, NAME_None, InValue, Type)
	{}

	FConfigValue(const FConfigSection* InSection, FName InValueName, const FString& InValue, EValueType Type = EValueType::Combined)
		: SavedValue(InValue)
		, ValueType(Type)
#if UE_WITH_CONFIG_TRACKING
		, SectionAccess(GetSectionAccess(InSection))
		, ValueName(InValueName)
#endif
	{
		SavedValueHash = FTextLocalizationResource::HashString(SavedValue);
		bExpandOnDemand = NeedsToExpandValue();
	}

	FConfigValue(FString&& InValue, EValueType Type=EValueType::Combined)
		: FConfigValue(nullptr, NAME_None, MoveTemp(InValue), Type)
	{}

	FConfigValue(const FConfigSection* InSection, FName InValueName, FString&& InValue, EValueType Type = EValueType::Combined)
		: SavedValue(MoveTemp(InValue))
		, ValueType(Type)
#if UE_WITH_CONFIG_TRACKING
		, SectionAccess(GetSectionAccess(InSection))
		, ValueName(InValueName)
#endif
	{
		SavedValueHash = FTextLocalizationResource::HashString(SavedValue);
		bExpandOnDemand = NeedsToExpandValue();
	}

	FConfigValue(const FConfigValue& InConfigValue)
		: SavedValue(InConfigValue.SavedValue)
		, SavedValueHash(InConfigValue.SavedValueHash)
		, ValueType(InConfigValue.ValueType)
		, bExpandOnDemand(InConfigValue.bExpandOnDemand)
#if UE_WITH_CONFIG_TRACKING
		, SectionAccess(InConfigValue.SectionAccess)
		, ValueName(InConfigValue.ValueName)
#endif
	{
		// shouldn't need to expand value it's assumed that the other FConfigValue has done this already
	}

	FConfigValue(FConfigValue&& InConfigValue)
		: SavedValue(MoveTemp(InConfigValue.SavedValue))
		, SavedValueHash(InConfigValue.SavedValueHash)
		, ValueType(InConfigValue.ValueType)
		, bExpandOnDemand(InConfigValue.bExpandOnDemand)
#if UE_WITH_CONFIG_TRACKING
		, SectionAccess(MoveTemp(InConfigValue.SectionAccess))
		, ValueName(InConfigValue.ValueName)
#endif
	{
		// shouldn't need to expand value it's assumed that the other FConfigValue has done this already
	}

	FConfigValue& operator=(FConfigValue&& RHS)
	{
		SavedValue = MoveTemp(RHS.SavedValue);
		SavedValueHash = RHS.SavedValueHash;
		ValueType = RHS.ValueType;
		bExpandOnDemand = RHS.bExpandOnDemand;
#if UE_WITH_CONFIG_TRACKING
		SectionAccess = MoveTemp(RHS.SectionAccess);
		ValueName = RHS.ValueName;
#endif

		return *this;
	}

	FConfigValue& operator=(const FConfigValue& RHS)
	{
		SavedValue = RHS.SavedValue;
		SavedValueHash = RHS.SavedValueHash;
		ValueType = RHS.ValueType;
		bExpandOnDemand = RHS.bExpandOnDemand;
#if UE_WITH_CONFIG_TRACKING
		SectionAccess = RHS.SectionAccess;
		ValueName = RHS.ValueName;
#endif

		return *this;
	}

	FConfigValue& operator=(const TCHAR* RHS)
	{
		*this = FString(RHS);
		return *this;
	}
	FConfigValue& operator=(const FString& RHS)
	{
		*this = FString(RHS);
		return *this;
	}
	FConfigValue& operator=(FString&& RHS)
	{
		SavedValue = MoveTemp(RHS);
		SavedValueHash = FTextLocalizationResource::HashString(SavedValue);
		bExpandOnDemand = NeedsToExpandValue();
		return *this;
	}

	// Returns the ini setting with any macros expanded out
	inline FString GetValue() const
	{
#if UE_WITH_CONFIG_TRACKING
		UE::ConfigAccessTracking::Private::OnConfigValueRead(SectionAccess, ValueName, *this);
#endif
		return bExpandOnDemand ? ExpandValue(SavedValue) : SavedValue;
	}

	// Returns the original ini setting without macro expansion
	const FString& GetSavedValue() const
	{
#if UE_WITH_CONFIG_TRACKING
		UE::ConfigAccessTracking::Private::OnConfigValueRead(SectionAccess, ValueName, *this);
#endif
		return SavedValue;
	}
#if UE_WITH_CONFIG_TRACKING
	void SetSectionAccess(UE::ConfigAccessTracking::FSection* InSectionAccess)
	{
		SectionAccess = InSectionAccess;
	}
#endif

	bool operator==(const FConfigValue& Other) const { return SavedValueHash == Other.SavedValueHash; }
	bool operator!=(const FConfigValue& Other) const { return !(FConfigValue::operator==(Other)); }

	friend FArchive& operator<<(FArchive& Ar, FConfigValue& ConfigValue)
	{
		FStructuredArchiveFromArchive(Ar).GetSlot() << ConfigValue;
		return Ar;
	}

	friend void operator<<(FStructuredArchive::FSlot Slot, FConfigValue& ConfigValue)
	{
		Slot << ConfigValue.SavedValue;

		if (Slot.GetUnderlyingArchive().IsLoading())
		{
			ConfigValue.bExpandOnDemand = ConfigValue.NeedsToExpandValue();
		}
	}

	/**
	 * Given a collapsed config value, try and produce an expanded version of it (removing any placeholder tokens).
	 *
	 * @param InCollapsedValue		The collapsed config value to try and expand.
	 * @param OutExpandedValue		String to fill with the expanded version of the config value.
	 *
	 * @return true if expansion occurred, false if the collapsed and expanded values are equal.
	 */
	CORE_API static bool ExpandValue(const FString& InCollapsedValue, FString& OutExpandedValue);

	/**
	 * Given a collapsed config value, try and produce an expanded version of it (removing any placeholder tokens).
	 *
	 * @param InCollapsedValue		The collapsed config value to try and expand.
	 *
	 * @return The expanded version of the config value.
	 */
	CORE_API static FString ExpandValue(const FString& InCollapsedValue);

	/**
	 * Given an expanded config value, try and produce a collapsed version of it (adding any placeholder tokens).
	 *
	 * @param InExpandedValue		The expanded config value to try and expand.
	 * @param OutCollapsedValue		String to fill with the collapsed version of the config value.
	 *
	 * @return true if collapsing occurred, false if the collapsed and expanded values are equal.
	 */
	CORE_API static bool CollapseValue(const FString& InExpandedValue, FString& OutCollapsedValue);

	/**
	 * Given an expanded config value, try and produce a collapsed version of it (adding any placeholder tokens).
	 *
	 * @param InExpandedValue		The expanded config value to try and expand.
	 *
	 * @return The collapsed version of the config value.
	 */
	CORE_API static FString CollapseValue(const FString& InExpandedValue);

#if CONFIG_CAN_SAVE_COMMENTS
	FString Comment;
#endif

	/** Gets the expanded value (GetValue) without marking it as having been accessed for e.g. writing out to a ConfigFile to disk */
	UE_FORCEINLINE_HINT FString GetValueForWriting() const
	{
		return bExpandOnDemand ? ExpandValue(SavedValue) : SavedValue;
	};

	/** Gets the SavedValue without marking it as having been accessed for e.g. writing out to a ConfigFile to disk */
	friend class FConfigCacheIni;
	friend class FConfigFile;
	friend class FConfigBranch;
	friend struct FDetailedConfigMemUsage;
	const FString& GetSavedValueForWriting() const
	{
		return SavedValue;
	};

private:
	/** Internal version of ExpandValue that expands SavedValue into ExpandedValue, or produces an empty ExpandedValue if no expansion occurred. */
	CORE_API bool NeedsToExpandValue();

	FString SavedValue;
	uint32 SavedValueHash;
public:
	// Add, subtract, stc
	EValueType ValueType;
private:
	bool bExpandOnDemand = false;
#if UE_WITH_CONFIG_TRACKING
	TRefCountPtr<UE::ConfigAccessTracking::FSection> SectionAccess;
	FMinimalName ValueName;
#endif
};

typedef TMultiMap<FName,FConfigValue> FConfigSectionMap;

// One section in a config file.
class FConfigSection : public FConfigSectionMap
{
public:
	FConfigSection()
		: FConfigSection(nullptr)
	{
	}

	FConfigSection(UE::ConfigAccessTracking::FSection* InSectionAccess)
#if UE_WITH_CONFIG_TRACKING
		: SectionAccess(InSectionAccess)
#endif
	{
	}

	/**
	* Check whether the input string is surrounded by quotes
	*
	* @param Test The string to check
	*
	* @return true if the input string is surrounded by quotes
	*/
	static bool HasQuotes( const FString& Test );
	bool operator==( const FConfigSection& Other ) const;
	bool operator!=( const FConfigSection& Other ) const;

	// process the '+' and '.' commands, takingf into account ArrayOfStruct unique keys
	void CORE_API HandleAddCommand(FName ValueName, FString&& Value, bool bAppendValueIfNotArrayOfStructsKeyUsed);

	bool HandleArrayOfKeyedStructsCommand(FName Key, FString&& Value);

	template<typename Allocator> 
	void MultiFind(const FName Key, TArray<FConfigValue, Allocator>& OutValues, const bool bMaintainOrder = false) const
	{
		FConfigSectionMap::MultiFind(Key, OutValues, bMaintainOrder);
	}

	template<typename Allocator> 
	void MultiFind(const FName Key, TArray<FString, Allocator>& OutValues, const bool bMaintainOrder = false) const
	{
		for (typename ElementSetType::TConstKeyIterator It(Pairs, Key); It; ++It)
		{
			OutValues.Add(It->Value.GetValue());
		}

		if (bMaintainOrder)
		{
			Algo::Reverse(OutValues);
		}
	}

	CORE_API bool GetString(const TCHAR* Key, FString& Value) const;
	CORE_API bool GetText(const TCHAR* Section, const TCHAR* Key, FText& Value) const;
	CORE_API bool GetInt(const TCHAR* Key, int32& Value) const;
	CORE_API bool GetUInt(const TCHAR* Key, uint32& Value) const;
	CORE_API bool GetFloat(const TCHAR* Key, float& Value) const;
	CORE_API bool GetDouble(const TCHAR* Key, double& Value) const;
	CORE_API bool GetInt64(const TCHAR* Key, int64& Value) const;
	CORE_API bool GetBool(const TCHAR* Key, bool& Value) const;
	CORE_API int32 GetArray(const TCHAR* Key, TArray<FString>& Value) const;

	// look for "array of struct" keys for overwriting single entries of an array
	TMap<FName, FString> ArrayOfStructKeys;
#if UE_WITH_CONFIG_TRACKING
	TRefCountPtr<UE::ConfigAccessTracking::FSection> SectionAccess;
#endif
	
	// a set of (array) keys that have been initialized to empty - this is needed to we can tell the difference between
	// wanting the array to be cleared out in LoadConfig, vs not being specified at all. a bool can be initialized to false
	// or not specified at all, but an array has no way to differentiate without this
	TSet<FName> EmptyInitializedKeys;

	// dyanmic modification will disable saving for this section
	bool bCanSave = true;
	
	friend FArchive& operator<<(FArchive& Ar, FConfigSection& ConfigSection);

private:
	friend FConfigFile;
	static bool AreSectionsEqualForWriting(const FConfigSection& A, const FConfigSection& B);
};

#if ALLOW_INI_OVERRIDE_FROM_COMMANDLINE
// Options which stemmed from the commandline
struct FConfigCommandlineOverride
{
	FString BaseFileName, Section, PropertyKey, PropertyValue;
};
#endif // ALLOW_INI_OVERRIDE_FROM_COMMANDLINE

typedef TMap<FString, FConfigSection> FConfigFileMap;

class FConfigModificationTracker
{
public:
	struct FCVarTracker
	{
		// this must be an ECVF priority
		int CVarPriority;
		// tag -> branch -> cvars
		TMap<FName, TMap<FName, FConfigSection>> CVarEntriesPerBranchPerTag;
	};
	
	// input
	bool bTrackModifiedSections = true;
	bool bTrackLoadedFiles = false;
	bool bIsUnloadingPlugin = false;
	// if this is true and bTrackLoadedFiles is set, then we ignore files in LoadedFiles array
	bool bSkipLoadingAlreadyLoadedFiles = false;

	// output
	TMap<FName, TSet<FString>> ModifiedSectionsPerBranch;
	TArray<FString> LoadedFiles;
	
	// section -> tracker
	TMap<FString, FCVarTracker> CVars;

	// sometimes reloading all instances/subclasses of a class causes trouble, so we 
	TArray<FString> ClassesToSkipSubclasses;
	TArray<FString> ClassesToSkipInstances;
};

class FConfigCommandStreamSection : public FConfigSectionMap
{
public:
	TMap<FName, FString> ArrayOfStructKeys;
};

// this ended up being the same as FConfigSection, but we use the different type to indicate these are always combined
class FConfigCommandStream : public TMap<FString, FConfigCommandStreamSection>
{
public:
	using SectionType = FConfigCommandStreamSection;
	
	void ProcessCommand(SectionType* Section, FStringView SectionName, FConfigValue::EValueType Command, FName Key, FString&& Value);
	FConfigCommandStreamSection* FindOrAddSectionInternal(const FString& SectionName);
	bool FillFileFromDisk(const FString& Filename, bool bHandleSymbolCommands);

	void Shrink();

	// This holds per-object config class names, with their ArrayOfStructKeys. Since the POC sections are all unique,
	// we can't track it just in that section. This is expected to be empty/small
	TMap<FString, TMap<FName, FString> > PerObjectConfigArrayOfStructKeys;
	
	class FConfigBranch* Branch = nullptr;
	FName Tag;
	
	uint8 Dirty;
	uint8 bPythonConfigParserMode = false;

	// used to determine if existing settings should be removed from a FConfigFile before applying this stream - used for compatibility with
	// how Saved config files are stored (replace the static layers values fully if at least one key exists)
	uint8 bIsSavedConfigFile = false;

	// we can't SafeUnload layers that come from a string, because we can't reload (Hotfixes, in particular)
	uint8 bNeverSafeUnload = false;
	
	uint16 Priority;
	FString Filename;
};

// One config file.
class FConfigFile : private FConfigFileMap
{
public:
	using SectionType = FConfigSection;
	
	bool Dirty : 1; // = false;
	bool NoSave : 1; // = false;
	bool bHasPlatformName : 1; // = false;
	bool bPythonConfigParserMode : 1; // = false;

	// by default, we allow saving - this is going to be applied to config files that are not loaded from disk
	// (when loading, this will get set to false, and then the ini sections will be checked)
	bool bCanSaveAllSections : 1; // = true;
#if UE_WITH_CONFIG_TRACKING
	UE::ConfigAccessTracking::ELoadType LoadType = UE::ConfigAccessTracking::ELoadType::Uninitialized;
#endif

	/** The name of this config file */
	FName Name;
	FString PlatformName;
	
	// Optional tag, (can tag files per plugin, etc)
	FName Tag;

	// this will point to the owning branch for the InMemoryFile only
	friend class FConfigBranch;
	class FConfigBranch* Branch = nullptr;
	
#if ALLOW_INI_OVERRIDE_FROM_COMMANDLINE
	/** The collection of overrides which stemmed from the commandline */
	TArray<FConfigCommandlineOverride> CommandlineOptions;
#endif // ALLOW_INI_OVERRIDE_FROM_COMMANDLINE

private:
	// This holds per-object config class names, with their ArrayOfStructKeys. Since the POC sections are all unique,
	// we can't track it just in that section. This is expected to be empty/small
	TMap<FString, TMap<FName, FString> > PerObjectConfigArrayOfStructKeys;
	
	// if this is set, then we track changes made to sections for saving or replaying later (currently unused)
	FConfigCommandStream* ChangeTracker = nullptr;
	
	// thread safety for reading while something is modifying this file
	FConfigSystemMutex SharedMutex;
	
#if UE_WITH_CONFIG_TRACKING
	mutable TRefCountPtr<UE::ConfigAccessTracking::FFile> FileAccess;
#endif

public:
	CORE_API FConfigFile();
	FConfigFile( int32 ) {}	// @todo UE-DLL: Workaround for instantiated TMap template during DLLExport (TMap::FindRef)
	CORE_API ~FConfigFile();
	CORE_API FConfigFile(const FConfigFile& Other);
	CORE_API FConfigFile(FConfigFile&& Other);
	CORE_API FConfigFile& operator=(const FConfigFile& Other);
	CORE_API FConfigFile& operator=(FConfigFile&& Other);

	// looks for a section by name, and creates an empty one if it can't be found
	CORE_API const FConfigSection* FindOrAddConfigSection(const FString& Name);

	inline const FConfigSection* FindSection(const FString& SectionName) const
	{
		UE::TReadScopeLock ScopeLock(ConfigFileMapLock);
		return FConfigFileMap::Find(SectionName);
	}

	inline int32 Num() const
	{
		UE::TReadScopeLock ScopeLock(ConfigFileMapLock);
		return FConfigFileMap::Num();
	}

	inline bool IsEmpty() const
	{
		UE::TReadScopeLock ScopeLock(ConfigFileMapLock);
		return FConfigFileMap::IsEmpty();
	}

	inline void Empty(int32 ExpectedNumElements = 0)
	{
		UE::TWriteScopeLock ScopeLock(ConfigFileMapLock);
		FConfigFileMap::Empty(ExpectedNumElements);
	}

	inline bool Contains(const FString& SectionName) const
	{
		UE::TReadScopeLock ScopeLock(ConfigFileMapLock);
		return FConfigFileMap::Contains(SectionName);
	}

	inline int32 GetKeys(TArray<FString>& Keys) const
	{
		UE::TReadScopeLock ScopeLock(ConfigFileMapLock);
		return FConfigFileMap::GetKeys(Keys);
	}

	inline int32 GetKeys(TSet<FString>& Keys) const
	{
		UE::TReadScopeLock ScopeLock(ConfigFileMapLock);
		return FConfigFileMap::GetKeys(Keys);
	}

	inline int32 Remove(KeyInitType InKey)
	{
		UE::TWriteScopeLock ScopeLock(ConfigFileMapLock);
		return FConfigFileMap::Remove(InKey);
	}

	inline ValueType& Add(const KeyType& InKey, const ValueType& InValue)
	{
		UE::TWriteScopeLock ScopeLock(ConfigFileMapLock);
		return FConfigFileMap::Add(InKey, InValue);
	}

	inline ValueType& Add(const KeyType&  InKey, ValueType&& InValue)
	{
		UE::TWriteScopeLock ScopeLock(ConfigFileMapLock);
		return FConfigFileMap::Add(InKey, MoveTempIfPossible(InValue));
	}

	inline ValueType& Add(KeyType&& InKey, const ValueType&  InValue)
	{
		UE::TWriteScopeLock ScopeLock(ConfigFileMapLock);
		return FConfigFileMap::Add(MoveTempIfPossible(InKey), InValue);
	}

	inline ValueType& Add(KeyType&& InKey,	ValueType&& InValue)
	{
		UE::TWriteScopeLock ScopeLock(ConfigFileMapLock);
		return FConfigFileMap::Add(MoveTempIfPossible(InKey), MoveTempIfPossible(InValue));
	}
	
	inline void Append(TMap<FString, FConfigSection> Other)
	{
		UE::TWriteScopeLock ScopeLock(ConfigFileMapLock);
		FConfigFileMap::Append(MoveTemp(Other));
	}

	inline void Reset()
	{
		UE::TWriteScopeLock ScopeLock(ConfigFileMapLock);
		FConfigFileMap::Reset();
	}

	inline TRangedForConstIterator begin() const
	{
		UE::TReadScopeLock ScopeLock(ConfigFileMapLock); 
		return TRangedForConstIterator(Pairs.begin());
	}

	inline TRangedForConstIterator end() const
	{
		UE::TReadScopeLock ScopeLock(ConfigFileMapLock);
		return TRangedForConstIterator(Pairs.end());
	}
	
	///////////////////////////////////

	bool operator==( const FConfigFile& Other ) const;
	bool operator!=( const FConfigFile& Other ) const;

	CORE_API bool Combine(const FString& Filename, bool bHandleSymbolCommands = true);
	CORE_API void CombineFromBuffer(const FString& Buffer, const FString& FileHint, bool bHandleSymbolCommands = true);
	CORE_API void Read( const FString& Filename );

	/**
	 * Apply the contents of the given file (which must have non-Combined ValueTypes stored in it, it is a logic error otherwise and will assert)
	 * The values in this file will be Combined type
	 */
	CORE_API bool ApplyFile(const FConfigCommandStream* File);

	/** Whether to write a temp file then move it to it's destination when saving. */
	CORE_API static bool WriteTempFileThenMove();

	/** Write this ConfigFile to the given Filename, constructed the text from the config sections in *this, prepended by the optional PrefixText */
	CORE_API bool Write( const FString& Filename, bool bDoRemoteWrite=true, const FString& PrefixText=FString());

	/** Write this ConfigFile to the given string, constructed the text from the config sections in *this, prepended by the optional PrefixText 
	 * @param SimulatedFilename - If writing a default hierarchal ini, can be used to correctly deduce position in the hierarchy
	 */
	CORE_API void WriteToString(FString& InOutText, const FString& SimulatedFilename = FString(), const FString& PrefixText = FString());

private:
	/** Determine if writing a default hierarchal ini, and deduce position in the hierarchy */
	bool IsADefaultIniWrite(const FString& Filename, int32& OutIniCombineThreshold) const;

	/** Write a ConfigFile to the given Filename, constructed from the given SectionTexts, in the given order, with sections in *this overriding sections in SectionTexts
	 * @param Filename - The file to write to
	 * @param bDoRemoteWrite - If true, also write the file to FRemoteConfig::Get()
	 * @param InOutSectionTexts - A map from section name to existing text for that section; text does not include the name of the section.
	 *  Entries in the TMap that also exist in *this will be updated.
	 *  If the empty string is present, it will be written out first (it is interpreted as a prefix before the first section)
	 * @param InSectionOrder - List of section names in the order in which each section should be written to disk, from e.g. the existing file.
	 *  Any section in this array that is not found in InOutSectionTexts will be ignored.
	 *  Any section in InOutSectionTexts that is not in this array will be appended to the end.
	 *  Duplicate entries are ignored; the first found index is used.
	 * @param OriginalContents - [optional] If set, and the string matches the generated string to write out, we skip the write to disk (it will return true)
	 * @return TRUE if the write was successful
	 */
	bool WriteInternal(const FString& Filename, bool bDoRemoteWrite, TMap<FString, FString>& InOutSectionTexts, const TArray<FString>& InSectionOrder, FString* OriginalContents);

	/** Write a ConfigFile to InOutText, constructed from the given SectionTexts, in the given order, with sections in *this overriding sections in SectionTexts
	 * @param InOutText - The string to write to
	 * @param bIsADefaultIniWrite - If true, force all properties to be written
	 * @param IniCombineThreshold - Cutoff level for combining ini (to prevent applying changes from the same ini that we're writing)
	 * @param InOutSectionTexts - A map from section name to existing text for that section; text does not include the name of the section.
	 *  Entries in the TMap that also exist in *this will be updated.
	 *  If the empty string is present, it will be written out first (it is interpreted as a prefix before the first section)
	 * @param InSectionOrder - List of section names in the order in which each section should be written to disk, from e.g. the existing file.
	 *  Any section in this array that is not found in InOutSectionTexts will be ignored.
	 *  Any section in InOutSectionTexts that is not in this array will be appended to the end.
	 *  Duplicate entries are ignored; the first found index is used.
	 * @return TRUE if the write was successful
	 */
	void WriteToStringInternal(FString& InOutText, bool bIsADefaultIniWrite, int32 IniCombineThreshold, TMap<FString, FString>& InOutSectionTexts, const TArray<FString>& InSectionOrder);

	/** Delete all of the inner ConfigFiles, for destruction or preparing to re-read
	 */
	void Cleanup();

	void Shrink();

	void FillFileFromBuffer(FStringView Buffer, bool bHandleSymbolCommands, const FString& FileHint);
	bool FillFileFromDisk(const FString& Filename, bool bHandleSymbolCommands);
	void ProcessCommand(FConfigSection* Section, FStringView SectionName, FConfigValue::EValueType Command, FName Key, FString&& Value);

	FConfigSection* FindOrAddSectionInternal(const FString& SectionName);

	inline FConfigSection* FindInternal(const FString& SectionName)
	{
		UE::TReadScopeLock ScopeLock(ConfigFileMapLock);
		return FConfigFileMap::Find(SectionName);
	};

	// allow the templated helper to access FindOrAddSectionInternal
	template<typename FileType>
	friend void FillFileFromBuffer(FileType* File, FStringView Buffer, bool bHandleSymbolCommands, const FString& FileHint);

public:
	CORE_API void Dump(FOutputDevice& Ar);

	bool GetString(const TCHAR* Section, const TCHAR* Key, FString& Value) const
	{
		if (const FConfigSection* ConfigSection = FindSection(Section))
		{
			return ConfigSection->GetString(Key, Value);
		}
		return false;
	}

	bool GetText(const TCHAR* Section, const TCHAR* Key, FText& Value) const
	{
		if (const FConfigSection* ConfigSection = FindSection(Section))
		{
			return ConfigSection->GetText(Section, Key, Value);
		}
		return false;
	}

	bool GetInt(const TCHAR* Section, const TCHAR* Key, int32& Value) const
	{
		if (const FConfigSection* ConfigSection = FindSection(Section))
		{
			return ConfigSection->GetInt(Key, Value);
		}
		return false;
	}

	bool GetUInt(const TCHAR* Section, const TCHAR* Key, uint32& Value) const
	{
		if (const FConfigSection* ConfigSection = FindSection(Section))
		{
			return ConfigSection->GetUInt(Key, Value);
		}
		return false;
	}

	bool GetFloat(const TCHAR* Section, const TCHAR* Key, float& Value) const
	{
		if (const FConfigSection* ConfigSection = FindSection(Section))
		{
			return ConfigSection->GetFloat(Key, Value);
		}
		return false;
	}

	bool GetDouble(const TCHAR* Section, const TCHAR* Key, double& Value) const
	{
		if (const FConfigSection* ConfigSection = FindSection(Section))
		{
			return ConfigSection->GetDouble(Key, Value);
		}
		return false;
	}

	bool GetInt64(const TCHAR* Section, const TCHAR* Key, int64& Value) const
	{
		if (const FConfigSection* ConfigSection = FindSection(Section))
		{
			return ConfigSection->GetInt64(Key, Value);
		}
		return false;
	}

	bool GetBool(const TCHAR* Section, const TCHAR* Key, bool& Value) const
	{
		if (const FConfigSection* ConfigSection = FindSection(Section))
		{
			return ConfigSection->GetBool(Key, Value);
		}
		return false;
	}

	CORE_API int32 GetArray(const TCHAR* Section, const TCHAR* Key, TArray<FString>& Value) const;

	/* Generic versions for use with templates */
	bool GetValue(const TCHAR* Section, const TCHAR* Key, FString& Value) const
	{
		return GetString(Section, Key, Value);
	}
	bool GetValue(const TCHAR* Section, const TCHAR* Key, FText& Value) const
	{
		return GetText(Section, Key, Value);
	}
	bool GetValue(const TCHAR* Section, const TCHAR* Key, int32& Value) const
	{
		return GetInt(Section, Key, Value);
	}
	bool GetValue(const TCHAR* Section, const TCHAR* Key, float& Value) const
	{
		return GetFloat(Section, Key, Value);
	}
	bool GetValue(const TCHAR* Section, const TCHAR* Key, double& Value) const
	{
		return GetDouble(Section, Key, Value);
	}
	bool GetValue(const TCHAR* Section, const TCHAR* Key, int64& Value) const
	{
		return GetInt64(Section, Key, Value);
	}
	bool GetValue(const TCHAR* Section, const TCHAR* Key, bool& Value) const
	{
		return GetBool(Section, Key, Value);
	}
	int32 GetValue(const TCHAR* Section, const TCHAR* Key, TArray<FString>& Value) const
	{
		return GetArray(Section, Key, Value);
	}

	CORE_API bool DoesSectionExist(const TCHAR* Section) const;

	CORE_API void SetString(const TCHAR* Section, const TCHAR* Key, const TCHAR* Value);
	CORE_API void SetText(const TCHAR* Section, const TCHAR* Key, const FText& Value);
	CORE_API void SetFloat(const TCHAR* Section, const TCHAR* Key, float Value);
	CORE_API void SetDouble(const TCHAR* Section, const TCHAR* Key, double Value);
	CORE_API void SetBool(const TCHAR* Section, const TCHAR* Key, bool Value);
	CORE_API void SetInt64(const TCHAR* Section, const TCHAR* Key, const int64 Value);
	CORE_API void SetArray(const TCHAR* Section, const TCHAR* Key, const TArray<FString>& Value);
	

	/**
	 * Sets the given key/value pair to the Section in the given File. This will replace any existing values (as if ResetKeyInSection had been called)
	 * @return true if the section was modified
	 */
	CORE_API bool SetInSection(const TCHAR* SectionName, FName Key, const FString& Value);
		
	/**
	 * Adds the given key/value pair to the Section. This will always add this pair to the section, even if the pair already exists.
	 * This is equivalent to the . operator in .ini files
	 * @return true if the section was modified
	 */
	CORE_API bool AddToSection(const TCHAR* Section, FName Key, const FString& Value);

	/**
	 * Adds the given key/value pair to the Section, if the pair didn't already exist
	 * This is equivalent to the + operator in .ini files
	 * @return true if the section was modified
	 */
	CORE_API bool AddUniqueToSection(const TCHAR* Section, FName Key, const FString& Value);

	/**
	 * Removes every entry in the Section that has Key, no matter what the Value is
	 * This is equivalent to the ! operator in .ini files
	 * @return true if the section was modified
	 */
	CORE_API bool RemoveKeyFromSection(const TCHAR* Section, FName Key);

	/**
	 * Removes every  entry in the Section that has the Key/Value pair
	 * This is equivalent to the - operator in .ini files (although it will remove all instances of the pair, not just a single one)
	 * @return true if the section was modified
	 */
	CORE_API bool RemoveFromSection(const TCHAR* Section, FName Key, const FString& Value);

	/**
	 * Similar to RemvoeKeyFromSection, but if this File's changes are being tracked, then we remove all changes to the
	 * the key that have been tracked. This would be used to remove enries in a layer so that the values from previous layers
	 * are used, unmodified. "Reset To Defaults"
	 * This is equivalent to deleting the keys from a .ini file
	 * @return true if the section was modified
	 */
	CORE_API bool ResetKeyInSection(const TCHAR* Section, FName Key);

	/**
	 * Process the contents of an .ini file that has been read into an FString
	 * 
	 * @param Filename Name of the .ini file the contents came from
	 * @param Contents Contents of the .ini file
	 */
	CORE_API void ProcessInputFileContents(FStringView Contents, const FString& FileHint);

	/** Adds any properties that exist in InSourceFile that this config file is missing */
	CORE_API void AddMissingProperties(const FConfigFile& InSourceFile);

	/**
	 * Saves only the sections in this FConfigFile into the given file. All other sections in the file are left alone. The sections in this
	 * file are completely replaced. If IniRootName is specified, the current section settings are diffed against the file in the hierarchy up
	 * to right before this file (so, if you are saving DefaultEngine.ini, and IniRootName is "Engine", then Base.ini and BaseEngine.ini
	 * will be loaded, and only differences against that will be saved into DefaultEngine.ini)
	 *
	 * ======================================
	 * @todo: This currently doesn't work with array properties!! It will output the entire array, and without + notation!!
	 * ======================================
	 *
	 * @param IniRootName the name (like "Engine") to use to load a .ini hierarchy to diff against
	 */
	CORE_API void UpdateSections(const TCHAR* DiskFilename, const TCHAR* IniRootName=nullptr, const TCHAR* OverridePlatform=nullptr);

	/**
	 * Update a single property in the config file, for the section that is specified.
	 */
	CORE_API bool UpdateSinglePropertyInSection(const TCHAR* DiskFilename, const TCHAR* PropertyName, const TCHAR* SectionName);

	
	/** 
	 * Check the source hierarchy which was loaded without any user changes from the Config/Saved dir.
	 * If anything in the default/base options have changed, we need to ensure that these propagate through
	 * to the final config so they are not potentially ignored
	 */
	void ProcessSourceAndCheckAgainstBackup();

	/** Checks if the PropertyValue should be exported in quotes when writing the ini to disk. */
	static bool ShouldExportQuotedString(const FString& PropertyValue);

	/** Generate a correctly escaped line to add to the config file for the given property */
	static FString GenerateExportedPropertyLine(const FString& PropertyName, const FString& PropertyValue);

	/** Append a correctly escaped line to add to the config file for the given property */
	static void AppendExportedPropertyLine(FString& Out, const FString& PropertyName, const FString& PropertyValue);

	/** Checks the command line for any overridden config settings */
	CORE_API static void OverrideFromCommandline(FConfigFile* File, const FString& Filename);
	CORE_API static void OverrideFromCommandline(FConfigCommandStream* File, const FString& Filename);

	/** Checks the command line for any overridden config file settings */
	CORE_API static bool OverrideFileFromCommandline(FString& Filename);

	/** Appends a new INI file to the SourceIniHierarchy and combines it with the current contents */
//	UE_DEPRECATED(5.4, "Use FConfigCacheIni::FindBranch (recommended) or FConfigFile.Branch (quick fix) to call AddDynamicLayerToHierarchy (or the other functions to add plugins to configs) ")
	CORE_API void AddDynamicLayerToHierarchy(const FString& Filename);

	friend FArchive& operator<<(FArchive& Ar, FConfigFile& ConfigFile);

#if UE_WITH_CONFIG_TRACKING
	CORE_API void SuppressReporting();
	CORE_API UE::ConfigAccessTracking::FFile* GetFileAccess() const;
#endif

private:
	/** 
	 * Save the source hierarchy which was loaded out to a backup file so we can check future changes in the base/default configs
	 */
	void SaveSourceToBackupFile();

	/** 
	 * Process the property for Writing to a default file. We will need to check for array operations, as default ini's rely on this
	 * being correct to function properly
	 *
	 * @param IniCombineThreshold - Cutoff level for combining ini (to prevent applying changes from the same ini that we're writing)
	 * @param InCompletePropertyToProcess - The complete property which we need to process for saving.
	 * @param OutText - The stream we are processing the array to
	 * @param SectionName - The section name the array property is being written to
	 * @param PropertyName - The property name of the array
	 */
	void ProcessPropertyAndWriteForDefaults(int32 IniCombineThreshold, const TArray<const FConfigValue*>& InCompletePropertyToProcess, FString& OutText, const FString& SectionName, const FString& PropertyName);

	/**
	 * Creates a chain of ini filenames to load and combine.
	 *
	 * @param InBaseIniName Ini name.
	 * @param InPlatformName Platform name, nullptr means to use the current platform
	 * @param OutHierarchy An array which is to receive the generated hierarchy of ini filenames.
	 */
	void AddStaticLayersToHierarchy(const TCHAR* InBaseIniName, const TCHAR* InPlatformName, const TCHAR* EngineConfigDir, const TCHAR* SourceConfigDir);
	static void AddStaticLayersToHierarchy(FConfigContext& Context);

	// for AddStaticLayersToHierarchy
	friend class FConfigCacheIni;
	friend FConfigContext;

	static CORE_API FTransactionallySafeRWLock ConfigFileMapLock;
};

/**
 * Declares a delegate type that's used by the config system to allow iteration of key value pairs.
 */
DECLARE_DELEGATE_TwoParams(FKeyValueSink, const TCHAR*, const TCHAR*);

enum class EConfigCacheType : uint8
{
	// this type of config cache will write its files to disk during Flush (i.e. GConfig)
	DiskBacked,
	// this type of config cache is temporary and will never write to disk (only load from disk)
	Temporary,
};

enum class EBranchReplayMethod : uint8
{
	// every file in the branch is saved with valuetypes, allowing for replay from beginning to end
	FullReplay,
	// store a copy of the staticlayers combined together, as a baseline for replaying dynamic layers after (useful for plugins to be removed)
	DynamicLayerReplay,
	// store only the final version of static + dynamic + saved
	NoReplay,
};


// this is the ordering within dynamic layers - for any given priority, the order is indeterminate (controlled by discovery of the 
// file on disk, or pulled down from hotfix server, etc)
enum class DynamicLayerPriority : uint16
{
	// unknown priority, lowest priority
	Unknown = 0,
	// plugins built into UE (a project can disable them, but these may want to override settings in a project plugin)
	EnginePlugin = 10,
	Plugin UE_DEPRECATED(5.8, "DynamicLayerPriority::Plugin is deprecated, use EnginePlugin or ProjectPlugin instead") = 20,
	// plugins in Project/Plugins, or external locations, that the project has enabled
	ProjectPlugin = 30,
	// this special layer allows for a .ini file specified in the project to override any plugin settings, just in cast that is needed
	ProjectPluginOverrides = 49,
	// GameFeaturePlugins tend to come and go, and expect to override other plugin settings
	GameFeature = 50,
	// hotfixes are generally going to the highest priority to be able to override any setting
	Hotfix = 80,
};

// A set of config files with the same "base name" like Engine, Input, or GameUserSettings. Contains the hierarchy of
// all possible files that will be searched for and the files that actually exist and can be loaded.
class FConfigBranch
{
public:
	// Standard branch that will be used by, say, GConfig to hold the inis for Engine, Game, etc
	FConfigBranch();
	
	// A "dummy" branch used to manage a single external FConfigFile
	FConfigBranch(const FConfigFile& ExistingFile);
	
	
	// base name of the branch, like "Engine"
	FName IniName;
	
	// "final" path for the branch like "Saved/Config/Windows/Engine.ini"
	FString IniPath;
	
	FName Platform;
	
	// Locations where this file may have come from - used to merge with non-standard ini locations
	FString SourceEngineConfigDir;
	FString SourceProjectConfigDir;
	
	bool bIsSafeUnloaded;
	bool bIsHierarchical;
	// If set true, this ConfigBranch will be allowed to be unloaded.
	bool bAllowedToRemove = true;

	EBranchReplayMethod ReplayMethod = EBranchReplayMethod::NoReplay;
	
	FConfigFileHierarchy Hierarchy;
	
	TMap<FString, FConfigCommandStream> StaticLayers;
	using DynamicLayerList = TDoubleLinkedList<FConfigCommandStream*>;
	DynamicLayerList DynamicLayers;
	FConfigCommandStream SavedLayer;
	
	// cache the static layers so when remaking dynamic layers after removing a dynamic layer it's faster
	FConfigFile CombinedStaticLayers;
	
	// this contains everything read from disk - when saving the diff between this and InMemoryFile is written out
	FConfigFile FinalCombinedLayers;
	
	FConfigCommandStream CommandLineOverrides;
	
	// this is the file that maps to the old FConfigFiles stored in the FConfigCacheIni
	FConfigFile InMemoryFile;
	
	// tracks runtime changes for optimal saving
	FConfigCommandStream RuntimeChanges;
	
	friend FArchive& operator<<(FArchive& Ar, FConfigBranch& ConfigBranch);
	
	/** Appends a new INI file to the SourceIniHierarchy and combines it with the current contents
	 * Additonally, returns the FConfigFIle object that contains just the loaded sections
	 * Can return the modified sections if the callers wants to reloadconfig on classes
	 */
	CORE_API bool AddDynamicLayerToHierarchy(const FString& Filename, FConfigModificationTracker* ModificationTracker=nullptr, const TSet<FString>* GlobalConfigFileCache=nullptr, const TSet<FString>* PluginConfigFileCache=nullptr);
	CORE_API bool AddDynamicLayersToHierarchy(const TArray<FDynamicLayerInfo>& Layers, FConfigModificationTracker* ModificationTracker=nullptr, const TSet<FString>* GlobalConfigFileCache=nullptr, const TSet<FString>* PluginConfigFileCache=nullptr, bool bForceFullDynamicLayerUpdate=false);

	// Add a preloaded string as a dynamic layer (useful for hotfixing)
	CORE_API bool AddDynamicLayerStringToHierarchy(const FString& Filename, const FString& Contents, FName Tag=NAME_None, DynamicLayerPriority Priority=DynamicLayerPriority::Unknown, FConfigModificationTracker* ModificationTracker=nullptr);
	
	/** Removes a dyanmic file from the hierarchy and recalculates the branch's IniMemoryFile
	 * Can return the modified sections if the callers wants to reloadconfig on classes
	 */
	CORE_API bool RemoveDynamicLayerFromHierarchy(const FString& Filename, FConfigModificationTracker* ModificationTracker=nullptr);
	CORE_API bool RemoveDynamicLayersFromHierarchy(const TArray<FString>& Filenames, FConfigModificationTracker* ModificationTracker=nullptr);
	
	/** 
	 * Frees up the static layer memory, which can be useful if a branch is loaded from, cached, and never used again. However,
	 * if something does try to access it, it will reload in-place (can hitch your game, so be aware)
	 */
	CORE_API void SafeUnload();
	CORE_API void SafeReload();

	/**
	 * Updates the inactive timer to the current time
	 */
	CORE_API void UpdateInactiveTimer();

	/** Recalculate the usable values from all static and dynamic layers. Useful if you update a layer and need to see results in 
	 * the InMemoryFile (ie what GConfig->GetXXX returns) 
	 */
	CORE_API void ReapplyLayers();

	/** 
	 * Apply all static layers up, but not including, the given layer name substring (so passing "DefaultEngine" will give all values before the DefaultEngine layer)
	 * This only works if static layers are being tracked (generally in the editor and not client builds - see EBranchReplayMethod::FullReplay)
	 * @return true if the layer was found, or false if the layer name was not found (which means all static layers will be combined and returned)
	 */
	CORE_API bool MergeStaticLayersUpTo(const FString& LayerNameSubstring, FConfigFile& OutFile) const;
	
	/**
	 * Same as MergeStaticLayersUpTo, but includes the passed in layer
	 */
	CORE_API bool MergeStaticLayersUpToAndIncluding(const FString& LayerNameSubstring, FConfigFile& OutFile) const;

	/**
	 * Find the static layer that first matches the substring (or full) name of the layer filename
	 */
	CORE_API const FConfigCommandStream* GetStaticLayer(const FString& LayerSubstring) const;

	/**
	 * Removes the section completely from all layers of this branch 
	 * This is destructive! It will not reload on demand (like SafeUnload does). Use this only when you know you will _never_ need
	 * the values in the section again!
	 * @return true if anything was removed
	 */
	CORE_API bool RemoveSection(const TCHAR* Section);

	/**
	 * Deletes the final saved branch file, such as "Saved/Config/Windows/Engine.ini".
	 * @return Whether the file was successfully deleted.
	 */
	CORE_API bool Delete();

	CORE_API void Flush();

	CORE_API void Shrink();

	CORE_API void Dump(FOutputDevice& Ar);


	/**
	 * Run a function on every file in the branch
	 */
	CORE_API void RunOnEachFile(TFunction<void(FConfigFile& File, const FString& Name)> Func);
	CORE_API void RunOnEachCommandStream(TFunction<void(FConfigCommandStream& File, const FString& Name)> Func);

private:
	void InitFiles();
	
	void RemoveTagsFromHierarchy(const TArray<FName>& Tags, FConfigModificationTracker* ModificationTracker);
	
	friend class FConfigCacheIni;
	
	// when we last Found the branch to pull data from it, this is used to unload after it's been unused for some time
	TSAN_ATOMIC(double) InactiveTimer = -1;
};


// Set of all cached config files.
class FConfigCacheIni
{
public:
	// Basic functions.
	CORE_API FConfigCacheIni(EConfigCacheType Type, FName PlatformName = NAME_None, bool bInGloballyRegistered = false);

	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */
	CORE_API FConfigCacheIni();

	CORE_API virtual ~FConfigCacheIni();

	/**
	* Disables any file IO by the config cache system
	*/
	CORE_API void DisableFileOperations();

	/**
	* Re-enables file IO by the config cache system
	*/
	CORE_API void EnableFileOperations();

	/**
	 * Returns whether or not file operations are disabled
	 */
	CORE_API bool AreFileOperationsDisabled();

	/**
	 * @return true after after the basic .ini files have been loaded
	 */
	bool IsReadyForUse()
	{
		return bIsReadyForUse;
	}

	/** Return whether this is one of the global ConfigSystems: GConfig or FConfigCacheIni::ForPlatform. */
	bool IsGloballyRegistered() const
	{
		return bGloballyRegistered;
	}

	/**
	 * Allow for periodic cleanup or other tasks
	 */
	CORE_API void Tick(float DeltaSeconds);

	/**
	* Prases apart an ini section that contains a list of 1-to-N mappings of strings in the following format
	*	 [PerMapPackages]
	*	 MapName=Map1
	*	 Package=PackageA
	*	 Package=PackageB
	*	 MapName=Map2
	*	 Package=PackageC
	*	 Package=PackageD
	* 
	* @param Section Name of section to look in
	* @param KeyOne Key to use for the 1 in the 1-to-N (MapName in the above example)
	* @param KeyN Key to use for the N in the 1-to-N (Package in the above example)
	* @param OutMap Map containing parsed results
	* @param Filename Filename to use to find the section
	*
	* NOTE: The function naming is weird because you can't apparently have an overridden function differnt only by template type params
	*/
	CORE_API void Parse1ToNSectionOfStrings(const TCHAR* Section, const TCHAR* KeyOne, const TCHAR* KeyN, TMap<FString, TArray<FString> >& OutMap, const FString& Filename);

	/**
	* Parses apart an ini section that contains a list of 1-to-N mappings of names in the following format
	*	 [PerMapPackages]
	*	 MapName=Map1
	*	 Package=PackageA
	*	 Package=PackageB
	*	 MapName=Map2
	*	 Package=PackageC
	*	 Package=PackageD
	*
	* @param Section Name of section to look in
	* @param KeyOne Key to use for the 1 in the 1-to-N (MapName in the above example)
	* @param KeyN Key to use for the N in the 1-to-N (Package in the above example)
	* @param OutMap Map containing parsed results
	* @param Filename Filename to use to find the section
	*
	* NOTE: The function naming is weird because you can't apparently have an overridden function differnt only by template type params
	*/
	CORE_API void Parse1ToNSectionOfNames(const TCHAR* Section, const TCHAR* KeyOne, const TCHAR* KeyN, TMap<FName, TArray<FName> >& OutMap, const FString& Filename);

	/**
	 * Finds the in-memory config file for a config cache filename.
	 *
	 * @param A known key like GEngineIni, or the return value of GetConfigFilename
	 *
	 * @return The existing config file or null if it does not exist in memory
	 */
	CORE_API FConfigFile* FindConfigFile(const FString& Filename);

	/**
	 * Finds, loads, or creates the in-memory config file for a config cache filename.
	 * 
	 * @param A known key like GEngineIni, or the return value of GetConfigFilename
	 * 
	 * @return A new or existing config file
	 */
	CORE_API FConfigFile* Find(const FString& InFilename);

	/**
	 * Reports whether an FConfigFile* is pointing to a config file inside of this
	 * Used for downstream functions to check whether a config file they were passed came from this ConfigCacheIni or from
	 * a different source such as LoadLocalIniFile
	 */
	CORE_API bool ContainsConfigFile(const FConfigFile* ConfigFile) const;

	/** Finds Config file that matches the base name such as "Engine" */
	CORE_API FConfigFile* FindConfigFileWithBaseName(FName BaseName);

	CORE_API FConfigFile& Add(const FString& Filename, const FConfigFile& File);

	/** Finds an FConfigBranch, using base name or a filename (can pass in NAME_None or empty string as needed) */
	CORE_API FConfigBranch* FindBranch(FName BaseIniName, const FString& Filename);

	/** Same as FindBranch, but this will not reload a formerly SafeUnload'd branch */
	CORE_API FConfigBranch* FindBranchWithNoReload(FName BaseIniName, const FString& Filename);

	/** Same as FindBranch, but only if the provided tag is found for one of its dynamic layers */
	CORE_API FConfigBranch* FindBranchWithDynamicLayerTag(FName BaseIniName, FName Tag);

	/** Create a new branch for FIlename, and return it */
	CORE_API FConfigBranch& AddNewBranch(const FString& Filename);
	
	CORE_API int32 Remove(const FString& Filename);
	
	CORE_API TArray<FString> GetFilenames();

	CORE_API bool Flush(bool bRemoveFromCache, const FString& Filename=TEXT(""));

	CORE_API void LoadFile( const FString& InFilename, const FConfigFile* Fallback = NULL, const TCHAR* PlatformString = NULL );
	CORE_API void SetFile( const FString& InFilename, const FConfigFile* NewConfigFile );
	CORE_API void UnloadFile( const FString& Filename );
	CORE_API void Detach( const FString& Filename );

	CORE_API bool GetString( const TCHAR* Section, const TCHAR* Key, FString& Value, const FString& Filename );
	CORE_API bool GetText( const TCHAR* Section, const TCHAR* Key, FText& Value, const FString& Filename );
	CORE_API bool GetSection( const TCHAR* Section, TArray<FString>& Result, const FString& Filename );
	CORE_API bool DoesSectionExist(const TCHAR* Section, const FString& Filename);

	CORE_API const FConfigSection* GetSection( const TCHAR* Section, const bool Force, const FString& Filename );
	CORE_API void SetString( const TCHAR* Section, const TCHAR* Key, const TCHAR* Value, const FString& Filename );
	CORE_API void SetText( const TCHAR* Section, const TCHAR* Key, const FText& Value, const FString& Filename );
	CORE_API bool RemoveKey( const TCHAR* Section, const TCHAR* Key, const FString& Filename );
	CORE_API bool EmptySection( const TCHAR* Section, const FString& Filename );
	CORE_API bool EmptySectionsMatchingString( const TCHAR* SectionString, const FString& Filename );

	/**
	 * For a base ini name, gets the config cache filename key that is used by other functions like Find.
	 * This will be the base name for known configs like Engine and the destination filename for others.
	 *
	 * @param IniBaseName Base name of the .ini (Engine, Game, CustomSystem)
	 *
	 * @return Filename key used by other cache functions
	 */
	CORE_API FString GetConfigFilename(const TCHAR* BaseIniName);

	/**
	 * Retrieve a list of all of the config files stored in the cache
	 *
	 * @param ConfigFilenames Out array to receive the list of filenames
	 */
	CORE_API void GetConfigFilenames(TArray<FString>& ConfigFilenames);

	/**
	 * Retrieve the names for all sections contained in the file specified by Filename
	 *
	 * @param	Filename			the file to retrieve section names from
	 * @param	out_SectionNames	will receive the list of section names
	 *
	 * @return	true if the file specified was successfully found;
	 */
	CORE_API bool GetSectionNames( const FString& Filename, TArray<FString>& out_SectionNames );

	/**
	 * Retrieve the names of sections which contain data for the specified PerObjectConfig class.
	 *
	 * @param	Filename			the file to retrieve section names from
	 * @param	SearchClass			the name of the PerObjectConfig class to retrieve sections for.
	 * @param	out_SectionNames	will receive the list of section names that correspond to PerObjectConfig sections of the specified class
	 * @param	MaxResults			the maximum number of section names to retrieve
	 *
	 * @return	true if the file specified was found and it contained at least 1 section for the specified class
	 */
	CORE_API bool GetPerObjectConfigSections( const FString& Filename, const FString& SearchClass, TArray<FString>& out_SectionNames, int32 MaxResults=1024 );

	CORE_API void Exit();

	/**
	 * Prints out the entire config set, or just a single file if an ini is specified
	 * 
	 * @param Ar the device to write to
	 * @param IniName An optional ini name to restrict the writing to (Engine or WrangleContent) - meant to be used with "final" .ini files (not Default*)
	 */
	CORE_API void Dump(FOutputDevice& Ar, const TCHAR* IniName=NULL);

	/**
	 * Dumps memory stats for each file in the config cache to the specified archive.
	 *
	 * @param	Ar	the output device to dump the results to
	 */
	CORE_API void ShowMemoryUsage( FOutputDevice& Ar );

	/**
	 * USed to get the max memory usage for the FConfigCacheIni
	 *
	 * @return the amount of memory in byes
	 */
	CORE_API SIZE_T GetMaxMemoryUsage();

	/**
	 * allows to iterate through all key value pairs
	 * @return false:error e.g. Section or Filename not found
	 */
	CORE_API bool ForEachEntry(const FKeyValueSink& Visitor, const TCHAR* Section, const FString& Filename);

	// Derived functions.
	CORE_API FString GetStr
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		const FString&	Filename
	);
	CORE_API bool GetInt
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		int32&				Value,
		const FString&	Filename
	);
	CORE_API bool GetInt64
	(
		const TCHAR* Section,
		const TCHAR* Key,
		int64& Value,
		const FString& Filename
	);
	CORE_API bool GetFloat
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		float&				Value,
		const FString&	Filename
	);
	CORE_API bool GetDouble
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		double&				Value,
		const FString&	Filename
	);
	CORE_API bool GetBool
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		bool&				Value,
		const FString&	Filename
	);
	CORE_API int32 GetArray
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		TArray<FString>&	out_Arr,
		const FString&	Filename
	);
	/** Loads a "delimited" list of strings
	 * @param Section - Section of the ini file to load from
	 * @param Key - The key in the section of the ini file to load
	 * @param out_Arr - Array to load into
	 * @param Filename - Ini file to load from
	 */
	CORE_API int32 GetSingleLineArray
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		TArray<FString>&	out_Arr,
		const FString&	Filename
	);
	CORE_API bool GetColor
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		FColor&				Value,
		const FString&	Filename
	);
	CORE_API bool GetVector2D(
		const TCHAR*   Section,
		const TCHAR*   Key,
		FVector2D&     Value,
		const FString& Filename);
	CORE_API bool GetVector
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		FVector&			Value,
		const FString&	Filename
	);
	CORE_API bool GetVector4
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		FVector4&			Value,
		const FString&	Filename
	);
	CORE_API bool GetRotator
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		FRotator&			Value,
		const FString&	Filename
	);

	/* Generic versions for use with templates */
	bool GetValue(const TCHAR* Section, const TCHAR* Key, FString& Value, const FString& Filename)
	{
		return GetString(Section, Key, Value, Filename);
	}
	bool GetValue(const TCHAR* Section, const TCHAR* Key, FText& Value, const FString& Filename)
	{
		return GetText(Section, Key, Value, Filename);
	}
	bool GetValue(const TCHAR* Section, const TCHAR* Key, int32& Value, const FString& Filename)
	{
		return GetInt(Section, Key, Value, Filename);
	}
	bool GetValue(const TCHAR* Section, const TCHAR* Key, float& Value, const FString& Filename)
	{
		return GetFloat(Section, Key, Value, Filename);
	}
	bool GetValue(const TCHAR* Section, const TCHAR* Key, bool& Value, const FString& Filename)
	{
		return GetBool(Section, Key, Value, Filename);
	}
	int32 GetValue(const TCHAR* Section, const TCHAR* Key, TArray<FString>& Value, const FString& Filename)
	{
		return GetArray(Section, Key, Value, Filename);
	}
		
	// Return a config value if found, if not found return default value
	// does not indicate if return value came from config or the default value
	// useful for one-time init of static variables in code locations where config may be queried too often, like :
	//  static int32 bMyConfigValue = GConfig->GetIntOrDefault(Section,Key,DefaultValue,ConfigFilename);
	int32 GetIntOrDefault(const TCHAR* Section, const TCHAR* Key, const int32 DefaultValue, const FString& Filename)
	{
		int32 Value = DefaultValue;
		GetInt(Section,Key,Value,Filename);
		return Value;
	}
	float GetFloatOrDefault(const TCHAR* Section, const TCHAR* Key, const float DefaultValue, const FString& Filename)
	{
		float Value = DefaultValue;
		GetFloat(Section,Key,Value,Filename);
		return Value;
	}
	bool GetBoolOrDefault(const TCHAR* Section, const TCHAR* Key, const bool DefaultValue, const FString& Filename)
	{
		bool Value = DefaultValue;
		GetBool(Section,Key,Value,Filename);
		return Value;
	}
	FString GetStringOrDefault(const TCHAR* Section, const TCHAR* Key, const FString & DefaultValue, const FString& Filename)
	{
		FString Value;
		if ( GetString(Section,Key,Value,Filename) )
		{
			return Value;
		}
		else
		{
			return DefaultValue;
		}
	}
	FText GetTextOrDefault(const TCHAR* Section, const TCHAR* Key, const FText & DefaultValue, const FString& Filename)
	{
		FText Value;
		if ( GetText(Section,Key,Value,Filename) )
		{
			return Value;
		}
		else
		{
			return DefaultValue;
		}
	}

	CORE_API void SetInt
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		int32					Value,
		const FString&	Filename
	);
	CORE_API void SetFloat
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		float				Value,
		const FString&	Filename
	);
	CORE_API void SetDouble
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		double				Value,
		const FString&	Filename
	);
	CORE_API void SetBool
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		bool				Value,
		const FString&	Filename
	);
	CORE_API void SetArray
	(
		const TCHAR*			Section,
		const TCHAR*			Key,
		const TArray<FString>&	Value,
		const FString&		Filename
	);
	/** Saves a "delimited" list of strings
	 * @param Section - Section of the ini file to save to
	 * @param Key - The key in the section of the ini file to save
	 * @param out_Arr - Array to save from
	 * @param Filename - Ini file to save to
	 */
	CORE_API void SetSingleLineArray
	(
		const TCHAR*			Section,
		const TCHAR*			Key,
		const TArray<FString>&	In_Arr,
		const FString&		Filename
	);
	CORE_API void SetColor
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		FColor				Value,
		const FString&	Filename
	);
	CORE_API void SetVector2D(
		const TCHAR*   Section,
		const TCHAR*   Key,
		FVector2D      Value,
		const FString& Filename);
	CORE_API void SetVector
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		FVector				Value,
		const FString&	Filename
	);
	CORE_API void SetVector4
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		const FVector4&		Value,
		const FString&	Filename
	);
	CORE_API void SetRotator
	(
		const TCHAR*		Section,
		const TCHAR*		Key,
		FRotator			Value,
		const FString&	Filename
	);
	

	/**
	 * Sets the given key/value pair to the Section in the given File. This will replace any existing values (as if ResetKeyInSection had been called)
	 * @return true if the section was modified
	 */
	CORE_API bool SetInSection(const TCHAR* Section, FName Key, const FString& Value, const FString& Filename);
	CORE_API bool SetInSectionOfStaticLayer(const TCHAR* Section, FName Key, const FString& Value, const FString& BranchName, const FString& LayerSubstring, bool bWriteFile);

	/**
	 * Adds the given key/value pair to the Section in the given File. This will always add this pair to the section, even if the pair already exists.
	 * This is equivalent to the . operator in .ini files
	 * @return true if the section was modified
	 */
	CORE_API bool AddToSection(const TCHAR* Section, FName Key, const FString& Value, const FString& Filename);
	CORE_API bool AddToSectionOfStaticLayer(const TCHAR* Section, FName Key, const FString& Value, const FString& BranchName, const FString& LayerSubstring, bool bWriteFile);

	/**
	 * Adds the given key/value pair to the Section in the given File, if the pair didn't already exist
	 * This is equivalent to the + operator in .ini files
	 * @return true if the section was modified
	 */
	CORE_API bool AddUniqueToSection(const TCHAR* Section, FName Key, const FString& Value, const FString& Filename);
	CORE_API bool AddUniqueToSectionOfStaticLayer(const TCHAR* Section, FName Key, const FString& Value, const FString& BranchName, const FString& LayerSubstring, bool bWriteFile);

	/**
	 * Removes every entry in the Section in the given File that has Key, no matter what the Value is
	 * This is equivalent to the ! operator in .ini files
	 * @return true if the section was modified
	 */
	CORE_API bool RemoveKeyFromSection(const TCHAR* Section, FName Key, const FString& Filename);
	CORE_API bool RemoveKeyFromSectionOfStaticLayer(const TCHAR* Section, FName Key, const FString& BranchName, const FString& LayerSubstring, bool bWriteFile);

	/**
	 * Removes every  entry in the Section in the given File that has the Key/Value pair
	 * This is equivalent to the - operator in .ini files (although it will remove all instances of the pair, not just a single one)
	 * @return true if the section was modified
	 */
	CORE_API bool RemoveFromSection(const TCHAR* Section, FName Key, const FString& Value, const FString& Filename);
	CORE_API bool RemoveFromSectionOfStaticLayer(const TCHAR* Section, FName Key, const FString& Value, const FString& BranchName, const FString& LayerSubstring, bool bWriteFile);

	/**
	 * Similar to RemvoeKeyFromSection, but if this File's changes are being tracked, then we remove all changes to the
	 * the key that have been tracked. This would be used to remove enries in a layer so that the values from previous layers
	 * are used, unmodified. "Reset To Defaults"
	 * This is equivalent to deleting the keys from a .ini file
	 * @return true if the section was modified
	 */
	CORE_API bool ResetKeyInSection(const TCHAR* Section, FName Key, const FString& Filename);
	CORE_API bool ResetKeyInSectionOfStaticLayer(const TCHAR* Section, FName Key, const FString& BranchName, const FString& LayerSubstring, bool bWriteFile);

	CORE_API bool SetArrayInSectionOfStaticLayer(const TCHAR* Section, FName Key, const TArray<FString>& Values, bool bClearArray, const FString& BranchName, const FString& LayerSubstring, bool bWriteFile);

	/**
	 * Clears out the memory of a branch, but can reload it on demand if needed
	 */
	CORE_API bool SafeUnloadBranch(const TCHAR* Filename);

	/**
	  * Removes the section completely from all layers of a branch. 
	  * This is destructive! It will not reload on demand (like SafeUnloadBranch does). Use this only when you know you will _never_ need
	  * the values in the section again!
	  * @return true if anything was removed
	 */
	CORE_API bool RemoveSectionFromBranch(const TCHAR* Section, const TCHAR* Filename);

	// Static helper functions

	/**
	 * Creates GConfig, loads the standard global ini files (Engine, Editor, etc),
	 * fills out GEngineIni, etc. and marks GConfig as ready for use
	 */
	static CORE_API void InitializeConfigSystem();

	/**
	 * Returns the Custom Config string, which if set will load additional config files from Config/Custom/{CustomConfig}/DefaultX.ini to allow different types of builds.
	 * It can be set from a game Target.cs file with CustomConfig = "Name".
	 * Or in development, it can be overridden with a -CustomConfig=Name command line parameter.
	 */
	static CORE_API const FString& GetCustomConfigString();

	/**
	 * Calculates the name of a dest (generated) .ini file for a given base (ie Engine, Game, etc)
	 *
	 * @param IniBaseName Base name of the .ini (Engine, Game)
	 * @param PlatformName Name of the platform to get the .ini path for (nullptr means to use the current platform)
	 * @param GeneratedConfigDir The base folder that will contain the generated config files.
	 *
	 * @return Standardized .ini filename
	 */
	static CORE_API FString GetDestIniFilename(const TCHAR* BaseIniName, const TCHAR* PlatformName, const TCHAR* GeneratedConfigDir);

	/**
	 * Loads and generates a destination ini file and adds it to GConfig:
	 *   - Looking on commandline for override source/dest .ini filenames
	 *   - Generating the name for the engine to refer to the ini
	 *   - Loading a source .ini file hierarchy
	 *   - Filling out an FConfigFile
	 *   - Save the generated ini
	 *   - Adds the FConfigFile to GConfig
	 *
	 * @param FinalIniFilename The output name of the generated .ini file (in Game\Saved\Config)
	 * @param BaseIniName The "base" ini name, with no extension (ie, Engine, Game, etc)
	 * @param Platform The platform to load the .ini for (if NULL, uses current)
	 * @param bForceReload If true, the destination .in will be regenerated from the source, otherwise this will only process if the dest isn't in GConfig
	 * @param bRequireDefaultIni If true, the Default*.ini file is required to exist when generating the final ini file.
	 * @param bAllowGeneratedIniWhenCooked If true, the engine will attempt to load the generated/user INI file when loading cooked games
	 * @param GeneratedConfigDir The location where generated config files are made.
	 * @return true if the final ini was created successfully.
	 */
	static CORE_API bool LoadGlobalIniFile(FString& FinalIniFilename, const TCHAR* BaseIniName, const TCHAR* Platform = NULL, bool bForceReload = false, bool bRequireDefaultIni = false, bool bAllowGeneratedIniWhenCooked = true, bool bAllowRemoteConfig = true, const TCHAR* GeneratedConfigDir = *FPaths::GeneratedConfigDir(), FConfigCacheIni* ConfigSystem=GConfig);

	/**
	 * Load an ini file directly into an FConfigFile, and nothing is written to GConfig or disk. 
	 * The passed in .ini name can be a "base" (Engine, Game) which will be modified by platform and/or commandline override,
	 * or it can be a full ini filename (ie WrangleContent) loaded from the Source config directory
	 *
	 * @param ConfigFile The output object to fill
	 * @param IniName Either a Base ini name (Engine) or a full ini name (WrangleContent). NO PATH OR EXTENSION SHOULD BE USED!
	 * @param bIsBaseIniName true if IniName is a Base name, which can be overridden on commandline, etc.
	 * @param Platform The platform to use for Base ini names, NULL means to use the current platform
	 * @param bForceReload force reload the ini file from disk this is required if you make changes to the ini file not using the config system as the hierarchy cache will not be updated in this case
	 * @return true if the ini file was loaded successfully
	 */
	static CORE_API bool LoadLocalIniFile(FConfigFile& ConfigFile, const TCHAR* IniName, bool bIsBaseIniName, const TCHAR* Platform=NULL, bool bForceReload=false);

	/**
	 * Load an ini file directly into an FConfigFile from the specified config folders, optionally writing to disk. 
	 * The passed in .ini name can be a "base" (Engine, Game) which will be modified by platform and/or commandline override,
	 * or it can be a full ini filename (ie WrangleContent) loaded from the Source config directory
	 *
	 * @param ConfigFile The output object to fill
	 * @param IniName Either a Base ini name (Engine) or a full ini name (WrangleContent). NO PATH OR EXTENSION SHOULD BE USED!
	 * @param EngineConfigDir Engine config directory.
	 * @param SourceConfigDir Game config directory.
	 * @param bIsBaseIniName true if IniName is a Base name, which can be overridden on commandline, etc.
	 * @param Platform The platform to use for Base ini names
	 * @param bForceReload force reload the ini file from disk this is required if you make changes to the ini file not using the config system as the hierarchy cache will not be updated in this case
	 * @param bWriteDestIni write out a destination ini file to the Saved folder, only valid if bIsBaseIniName is true
	 * @param bAllowGeneratedIniWhenCooked If true, the engine will attempt to load the generated/user INI file when loading cooked games
	 * @param GeneratedConfigDir The location where generated config files are made.
	 * @return true if the ini file was loaded successfully
	 */
	static CORE_API bool LoadExternalIniFile(FConfigFile& ConfigFile, const TCHAR* IniName, const TCHAR* EngineConfigDir, const TCHAR* SourceConfigDir, bool bIsBaseIniName, const TCHAR* Platform=NULL, bool bForceReload=false, bool bWriteDestIni=false, bool bAllowGeneratedIniWhenCooked = true, const TCHAR* GeneratedConfigDir = *FPaths::GeneratedConfigDir());

	/**
	 * Needs to be called after GConfig is set and LoadCoalescedFile was called.
	 * Loads the state of console variables.
	 * Works even if the variable is registered after the ini file was loaded.
	 */
	static CORE_API void LoadConsoleVariablesFromINI();

	/**
	 * Normalizes file paths to INI files.
	 *
	 * If an INI file is accessed with multiple paths, then we can run into issues where we cache multiple versions
	 * of the file. Specifically, any updates to the file may only be applied to one cached version, and could cause
	 * changes to be lost.
	 *
	 * E.G.
	 *
	 *		// Standard path.
	 *		C:\ProjectDir\Engine\Config\DefaultEngine.ini
	 *
	 *		// Lowercase drive, and an extra slash between ProjectDir and Engine.
	 *		c:\ProjectDir\\Engine\Confg\DefaultEngine.ini
	 *
	 *		// Relative to a project binary.
	 *		..\..\..\ConfigDefaultEngine.ini
	 *
	 *		The paths above could all be used to reference the same ini file (namely, DefaultEngine.ini).
	 *		However, they would end up generating unique entries in the GConfigCache.
	 *		That means any modifications to *one* of the entries would not propagate to the others, and if
	 *		any / all of the ini files are saved, they will stomp changes to the other entries.
	 *
	 *		We can prevent these types of issues by enforcing normalized paths when accessing configs.
	 *
	 *	@param NonNormalizedPath	The path to the INI file we want to access.
	 *	@return A normalized version of the path (may be the same as the input).
	 */
	static CORE_API FString NormalizeConfigIniPath(const FString& NonNormalizedPath);

	/**
	 * This helper function searches the cache before trying to load the ini file using LoadLocalIniFile. 
	 * Note that the returned FConfigFile pointer must have the same lifetime as the passed in LocalFile.
	 *
	 * @param LocalFile The output object to fill. If the FConfigFile is found in the cache, this won't be used.
	 * @param IniName Either a Base ini name (Engine) or a full ini name (WrangleContent). NO PATH OR EXTENSION SHOULD BE USED!
	 * @param Platform The platform to use for Base ini names, NULL means to use the current platform
	 * @return FConfigFile Found or loaded FConfigFile
	 */
	static CORE_API FConfigFile* FindOrLoadPlatformConfig(FConfigFile& LocalFile, const TCHAR* IniName, const TCHAR* Platform = NULL);

	/**
	 * Attempts to find the platform config in the cache.
	 *
	 * @param IniName Either a Base ini name (Engine) or a full ini name (WrangleContent). NO PATH OR EXTENSION SHOULD BE USED!
	 * @param Platform The platform to use for Base ini names, NULL means to use the current platform
	 * @return FConfigFile Found FConfigFile
	 */
	static CORE_API FConfigFile* FindPlatformConfig(const TCHAR* IniName, const TCHAR* Platform);
	
	/**
	 * Save the current config cache state into a file for bootstrapping other processes.
	 */
	CORE_API void SaveCurrentStateForBootstrap(const TCHAR* Filename);

	CORE_API void Serialize(FArchive& Ar);


	struct FKnownConfigFiles
	{
		FKnownConfigFiles();

		// Write out this for binary config serialization
		friend FArchive& operator<<(FArchive& Ar, FKnownConfigFiles& Names);

		// setup GEngineIni based on this structure's values
		void SetGlobalIniStringsFromMembers();

		// given an name ("Engine") return the FConfigFile for it
		const FConfigFile* GetFile(FName Name);
		// given an name ("Engine") return the modifiable FConfigFile for it
		FConfigFile* GetMutableFile(FName Name);

		// given an name ("Engine") return the modifiable FConfigBranch
		FConfigBranch* GetBranch(FName Name);

		// get the disk-based filename for the given known ini name
		const FString& GetFilename(FName Name);

		// the list of the known inis (Engine, Game, etc) See the top of this file for the list
		FConfigBranch Branches[(uint8)EKnownIniFile::NumKnownFiles];
	};

	/**
	 * Load the standard (used on all platforms) ini files, like Engine, Input, etc
	 *
	 * @param FConfigContext The loading context that controls the destination of the loaded ini files
	 *
	 * return True if the engine ini was loaded
	 */
	static CORE_API bool InitializeKnownConfigFiles(FConfigContext& Context);

	/**
	 * Returns true if the given name is one of the known configs, where the matching G****Ini property is going to match the 
	 * base name ("Engine" returns true, which means GEngineIni's value is just "Engine")
	 */
	CORE_API bool IsKnownConfigName(FName ConfigName);

	/**
	 * Create GConfig from a saved file
	 */
	static CORE_API bool CreateGConfigFromSaved(const TCHAR* Filename);

	/**
	 * Retrieve the fully processed ini system for another platform. The editor will start loading these 
	 * in the background on startup
	 */
	static CORE_API FConfigCacheIni* ForPlatform(FName PlatformName);

	/**
	 * Wipe all cached platform configs. Next ForPlatform call will load on-demand the platform configs
	 */
	static CORE_API void ClearOtherPlatformConfigs();

	/**
	 * Tell GConfig and the ForPlatform config systems about a plugin location
	 */
	static CORE_API void RegisterPlugin(FName PluginName, const FString& PluginDir, const TArray<FString>& ChildPluginDirs, DynamicLayerPriority Priority, bool bIncludePluginNameInBranchName);

	static CORE_API void AddPluginToAllBranches(FName PluginName, FConfigModificationTracker* ModificationTracker=nullptr);
	static CORE_API void AddMultiplePluginsToAllBranches(const TArray<FName>& PluginName, FConfigModificationTracker* ModificationTracker=nullptr);
	static CORE_API void RemoveTagFromAllBranches(FName Tag, FConfigModificationTracker* ModificationTracker=nullptr);
	static CORE_API void RemoveMultipleTagsFromAllBranches(const TArray<FName>& Tags, FConfigModificationTracker* ModificationTracker=nullptr);

	/**
	* Initializes configs and plugins for every supported platform on worker threads. This function can be
	* called prior to normal platform initialization to speed up the full initialization process.
	*/
	static CORE_API void PreInitializePlatformPlugins();

	/**
	 * Finds the Staged Plugin ConfigCache if it exists from a PluginName
	 *   Returns a pointer to the cache if it exists, a nullptr if not
	 */
	CORE_API const TSet<FString>* GetStagedPluginConfigCache(FName PluginName) const;

	/**
	 * Returns the staged global config cache
	 */
	CORE_API const TSet<FString>* GetStagedGlobalConfigCache() const;

private:
#if WITH_EDITOR
	/** We only auto-initialize other platform configs in the editor to not slow down programs like ShaderCOmpileWorker */
	static CORE_API void AsyncInitializeConfigForPlatforms();
#endif

	/** Serialize a bootstrapping state into or from an archive */
	CORE_API void SerializeStateForBootstrap_Impl(FArchive& Ar);


	void AddPluginsToBranches(const TArray<FName>& PluginNames, FConfigModificationTracker* ModificationTracker);
	void RemoveTagsFromBranches(const TArray<FName>& Tags, FConfigModificationTracker* ModificationTracker);

	/** Does this ConfigBranch IniName appear in our disallowed removal list **/
	bool IsConfigBranchNameInNeverUnloadList(const FName& ConfigBranchName);

	/** true if file operations should not be performed */
	bool bAreFileOperationsDisabled;

	/** true after the base .ini files have been loaded, and GConfig is generally "ready for use" */
	bool bIsReadyForUse;

	bool bGloballyRegistered;
	
	/** The type of the cache (basically, do we call Flush in the destructor) */
	EConfigCacheType Type;

	/** The platform this config system is for - if empty, then it can't be used with dynamic plugin layering */
	FName PlatformName;

	/** The filenames for the known files in this config */
	FKnownConfigFiles KnownFiles;

	TMap<FString, FConfigBranch*> OtherFiles;
	mutable FTransactionallySafeCriticalSection OtherFilesLock;
	
	/** Parallel array to OtherFiles */
	TArray<FString> OtherFileNames;

	struct FPluginInfo
	{
		FString PluginDir;
		TArray<FString> ChildPluginDirs;
		DynamicLayerPriority Priority;
		// packing in with priority
		uint8 bIncludePluginNameInBranchName : 1;
	};
	static TMap<FName, FPluginInfo*> RegisteredPlugins;
	TArray<FName> PendingModificationPlugins;
	static FTransactionallySafeCriticalSection RegisteredPluginsLock;

	// a cache discovered at staging time and loaded from BinaryConfig.ini
	TSet<FString>* StagedGlobalConfigCache = nullptr;
	TMap<FName, TSet<FString>> StagedPluginConfigCache;
	friend bool MakeBinaryConfig(const TCHAR* CmdLine);// @todo remove this and move the code that fills this out, maybe


#if ALLOW_OTHER_PLATFORM_CONFIG
	static TMap<FName, FConfigCacheIni*> ConfigForPlatform;
	static FCriticalSection ConfigForPlatformLock;
#endif

	/** List of ConfigBranch names that should never be unloaded. Read from parsing CVar. See ini.ConfigBranchesToNeverUnload **/
	TArray<FString> ConfigBranchNamesToNeverUnload;
	
	friend FConfigContext;
};


#if UE_WITH_CONFIG_TRACKING
inline UE::ConfigAccessTracking::FSection* GetSectionAccess(const FConfigSection* InSection)
{
	return InSection ? InSection->SectionAccess.GetReference() : nullptr;
}
#endif
