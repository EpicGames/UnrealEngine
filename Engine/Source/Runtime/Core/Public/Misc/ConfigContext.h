// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Misc/ConfigTypes.h"
// @todo mvoe types into ConfigTypes.h so we don't need to include the monster here
#include "Misc/ConfigCacheIni.h"


#ifndef CUSTOM_CONFIG
#define CUSTOM_CONFIG ""
#endif

#ifndef DISABLE_GENERATED_INI_WHEN_COOKED
#define DISABLE_GENERATED_INI_WHEN_COOKED 0
#endif


class FConfigCacheIni;
class FConfigFile;
class FConfigBranch;

class FConfigContext
{
public:

	/**
	 * Create a context to read a hierarchical config into the given local FConfigFile, optionally for another platform
	 * 
	 */
	static FConfigContext ReadIntoLocalFile(FConfigFile& DestConfigFile, const FString& Platform=FString())
	{
		return FConfigContext(nullptr, true, Platform, &DestConfigFile);
	}

	/**
	 * Create a context to read a non-hierarchical config into the given local FConfigFile, optionally for another platform
	 */
	static FConfigContext ReadSingleIntoLocalFile(FConfigFile& DestConfigFile, const FString& Platform=FString())
	{
		return FConfigContext(nullptr, false, Platform, &DestConfigFile);
	}

	/**
	 * Create a context to read a hierarchical config into GConfig. Only for current platform.
	 */
	static FConfigContext ReadIntoGConfig()
	{
		return FConfigContext(GConfig, true, FString());
	}

	/**
	 * Create a context to read a hierarchical config into GConfig. Only for current platform.
	 */
	static FConfigContext ForceReloadIntoGConfig()
	{
		FConfigContext Context(GConfig, true, FString());
		Context.bForceReload = true;
		return Context;
	}

	/**
	 * Create a context to read a non-hierarchical config into GConfig. Only for current platform.
	 */
	static FConfigContext ReadSingleIntoGConfig()
	{
		return FConfigContext(GConfig, false, FString());
	}

	/**
	 * Create a context to read a hierarchical config into the given ConfigSystem structure - usually will be for other platforms
	 */
	static FConfigContext ReadIntoConfigSystem(FConfigCacheIni* ConfigSystem, const FString& Platform)
	{
		return FConfigContext(ConfigSystem, true, Platform);
	}

	/**
	 * Create a context to read a non-hierarchical config into the given ConfigSystem structure - usually will be for other platforms
	 */
	static FConfigContext ReadSingleIntoConfigSystem(FConfigCacheIni* ConfigSystem, const FString& Platform)
	{
		return FConfigContext(GConfig, false, Platform);
	}

	/**
	 * Create a context to read a plugin's ini file named for the plugin. This is not used for inserting, say, Engine.ini into GConfig
	 */
	static FConfigContext ReadIntoPluginFile(FConfigFile& DestConfigFile, const FString& PluginRootDir, const TArray<FString>& ChildPluginsBaseDirs, const FString& Platform = FString())
	{
		FConfigContext Context(nullptr, true, Platform, &DestConfigFile);
		Context.bIsForPlugin = true;
		Context.PluginRootDir = PluginRootDir;
		Context.ChildPluginBaseDirs = ChildPluginsBaseDirs;


		// plugins are read in parallel, so we are reading into a file, but not touching GConfig, so bWriteDest would be false, but we
		// want to write them out as if we had been using GConfig
		// @todo: honestly, that's just to keep same behavior, but i am not sure it is correct behavior! actually why would reading in ever need to write out?
		Context.bWriteDestIni = true;

		return Context;
	}

	/**
	  * Inserts plugin ini files into an existing Branch
	 */
	static FConfigContext ReadPluginIntoConfigSystem(FConfigCacheIni* ConfigSystem, FName PluginName, const FString& PluginRootDir, const TArray<FString>& ChildPluginsBaseDirs)
	{
		FConfigContext Context(ConfigSystem, true, FString());
		Context.bIsForPlugin = true;
		Context.PluginRootDir = PluginRootDir;
		Context.ChildPluginBaseDirs = ChildPluginsBaseDirs;
		Context.ConfigFileTag = PluginName;

		return Context;
	}

	/**
	  * Inserts plugin ini files into an existing Branch
	 */
	static FConfigContext ReadPluginToModifyConfigSystem(FConfigCacheIni* ConfigSystem, DynamicLayerPriority Priority, FName PluginName, const FString& PluginRootDir, const TArray<FString>& ChildPluginsBaseDirs, FConfigModificationTracker* ChangeTracker=nullptr)
	{
		FConfigContext Context(ConfigSystem, true, FString());
		Context.bIsForPlugin = true;
		Context.bIsForPluginModification = true;
		Context.PluginRootDir = PluginRootDir;
		Context.ChildPluginBaseDirs = ChildPluginsBaseDirs;
		Context.ConfigFileTag = PluginName;
		Context.ChangeTracker = ChangeTracker;
		Context.PluginModificationPriority = Priority;

		return Context;
	}

	/**
	 * Create a context to read a hierarchy, but once it reaches the given filename (StartDeletingFilename), it will not read in anymore 
	 * files at that point
	 */
	static FConfigContext ReadUpToBeforeFile(FConfigFile& DestConfigFile, const FString& Platform, const FString& StartSkippingAtFilename)
	{
		FConfigContext Context(nullptr, true, Platform, &DestConfigFile);
		Context.StartSkippingAtFilename = StartSkippingAtFilename;
		return Context;
	}

	/**
	 * Create a context to read only the saved/generated file (eg. <Project>/Saved/Windows/Input.ini) and commandline options after loading
	 * a binary config, that was of course made without saved files and commandline options
	 */
	static FConfigContext FixupBranchAfterBinaryConfig()
	{
		FConfigContext Context(GConfig, true, FString());
		Context.bIsFixingUpAfterBinaryConfig = true;
		Context.bForceReload = true;
		return Context;
	}



	/**
	 * Call to make before attempting parallel config init
	 */
	static CORE_API void EnsureRequiredGlobalPathsHaveBeenInitialized();
	
	/** Log out the config full hierarchy of a file, with various overrides to see other projects/platforms/etc hierarchies */
	static CORE_API void VisualizeHierarchy(FOutputDevice& Ar, const TCHAR* IniName, const TCHAR* OverridePlatform, const TCHAR* OverrideProjectOrProgramDataDir, const TCHAR* OverridePluginDir=nullptr, const TArray<FString>* ChildPluginBaseDirs=nullptr);
	
	/** Visualize an existing hierarchy */
	void CORE_API VisualizeHierarchy(FOutputDevice& Ar, const TCHAR* IniName);

	/**
	 * Use the context to perform the actual load operation. Note that this is where you specify the Ini name (for instance "Engine"), meaning
	 * you can use the same context for multiple configs (Engine, Game, Input, etc)
	 */
	CORE_API bool Load(const TCHAR* IniName);

	/**
	 * Use the context to perform the actual load operation as above, but will return the generated final ini filename (in the case of GConfig, this would
	 * be the key used to look up into GConfig, for example)
	 */
	CORE_API bool Load(const TCHAR* IniName, FString& OutFilename);


	// because the hierarchy can jump between platforms, we cache off some directories per chained-platform
	struct FPerPlatformDirs
	{
		FString PlatformExtensionEngineDir;
		FString PlatformExtensionProjectDir;
		FString PlatformExtensionPluginDir;
	};
	/**
	 * Return the paths to use to find hierarchical config files for the given platform (note that these are independent of the ini name)
	 */
	CORE_API const FPerPlatformDirs& GetPerPlatformDirs(const FString& PlatformName);

	CORE_API ~FConfigContext();


	// @todo make these private and friend the FCOnfigCacheIni and FConfigFile once everything is a member function!!!!
	FConfigCacheIni* ConfigSystem;

	FConfigFile* ExistingFile = nullptr;
	FConfigBranch* Branch = nullptr;
	FConfigBranch* TemporaryBranch = nullptr;
	
	FString DestIniFilename;
	FString Platform;
	FString SavePlatform;
	FString GeneratedConfigDir;
	FString BaseIniName;
	FString StartSkippingAtFilename;

	FString EngineConfigDir;
	FString EngineRootDir;
	FString ProjectConfigDir;
	FString ProjectRootDir;
	FString PluginRootDir;
	TArray<FString> ChildPluginBaseDirs;
	FConfigModificationTracker* ChangeTracker = nullptr;
	
	FName ConfigFileTag;
	
	// useful strings that are used alot when walking the hierarchy
	FString ProjectLimitedAccessDir;
	FString ProjectNotForLicenseesDir;
	FString ProjectNoRedistDir;
	TMap<FString, FPerPlatformDirs> PerPlatformDirs;

	// allow a custom set of layers
	TArray<FConfigLayer> OverrideLayers;
	
	bool bUseHierarchyCache = false;
	bool bAllowGeneratedIniWhenCooked = false;
	bool bForceReload = false;
	bool bAllowRemoteConfig = false;
	bool bIsHierarchicalConfig;
	bool bWriteDestIni = false;
	bool bDefaultEngineRequired = false;
	bool bIsForPlugin = false;
	bool bIsForPluginModification = false;
	// GameFeaturePlugins have WindowsFooGame.ini, not WindowsGame.ini
	bool bIncludeTagNameInBranchName = false;
	bool bIsMakingBinaryConfig = false;
	bool bIsFixingUpAfterBinaryConfig = false;
	bool bCanRemoveBranchIfEmpty = false;
	bool bIsSafeReloading = false;

	DynamicLayerPriority PluginModificationPriority;

	// if this is non-null, it contains a set of pre-scanned ini files to use to find files, instead of looking on disk
	const TSet<FString>* IniCacheSet = nullptr;
	const TSet<FString>* StagedGlobalConfigCache = nullptr;
	const TSet<FString>* StagedPluginConfigCache = nullptr;
	
	TFunction<void(const TArray<FDynamicLayerInfo>&)> HandleLayersFunction = nullptr;

protected:

	bool bDoNotResetConfigFile = false;
	bool bCacheOnNextLoad = true;

	CORE_API FConfigContext(FConfigCacheIni* InConfigSystem, bool InIsHierarchicalConfig, const FString& InPlatform, FConfigFile* DestConfigFile=nullptr);

private:

	FConfigContext& ResetBaseIni(const TCHAR* InBaseIniName);
	void CachePaths();

	bool PrepareForLoad(bool& bPerformLoad);
	bool PerformLoad();
	bool PerformSingleFileLoad();

	void AddStaticLayersToHierarchy(TArray<FString>* GatheredLayerFilenames=nullptr, bool bIsForLogging=false);
	bool LoadIniFileHierarchy();
	bool GenerateDestIniFile();

	FString PerformFinalExpansions(const FString& InString, const FString& Platform);

	void LogVariables(const TCHAR* InBaseIniName, const FString& Platform);
};

bool DoesConfigFileExistWrapper(const TCHAR* IniFile, const TSet<FString>* IniCacheSet=nullptr, const TSet<FString>* PrimaryConfigFileCache = nullptr, const TSet<FString>* SecondaryConfigFileCache = nullptr);
