// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ConfigTypes.h"

// See FConfigContext.cpp for the types here

inline FConfigLayer GConfigLayers[] =
{
	/**************************************************
	**** CRITICAL NOTES
	**** If you change this array, you need to also change EnumerateConfigFileLocations() in ConfigHierarchy.cs!!!
	**** And maybe UObject::GetDefaultConfigFilename(), UObject::GetGlobalUserConfigFilename()
	**************************************************/

	// Engine/Base.ini
	{ TEXT("AbsoluteBase"),				TEXT("{ENGINE}/Config/Base.ini"), EConfigLayerFlags::NoExpand},

	// Engine/Base*.ini
	{ TEXT("Base"),						TEXT("{ENGINE}/Config/Base{TYPE}.ini"), EConfigLayerFlags::UseGlobalConfigCache },
	// Engine/Platform/BasePlatform*.ini
	{ TEXT("BasePlatform"),				TEXT("{ENGINE}/Config/{PLATFORM}/Base{PLATFORM}{TYPE}.ini"), EConfigLayerFlags::UseGlobalConfigCache  },
	// Project/Default*.ini
	{ TEXT("ProjectDefault"),			TEXT("{PROJECT}/Config/Default{TYPE}.ini"), EConfigLayerFlags::AllowCommandLineOverride | EConfigLayerFlags::UseGlobalConfigCache },
	// Project/Generated*.ini Reserved for files generated by build process and should never be checked in 
	{ TEXT("ProjectGenerated"),			TEXT("{PROJECT}/Config/Generated{TYPE}.ini"), EConfigLayerFlags::UseGlobalConfigCache },
	// Project/Custom/CustomConfig/Default*.ini only if CustomConfig is defined
	{ TEXT("CustomConfig"),				TEXT("{PROJECT}/Config/Custom/{CUSTOMCONFIG}/Default{TYPE}.ini"), EConfigLayerFlags::RequiresCustomConfig | EConfigLayerFlags::UseGlobalConfigCache },
	// Engine/Platform/Platform*.ini
	{ TEXT("EnginePlatform"),			TEXT("{ENGINE}/Config/{PLATFORM}/{PLATFORM}{TYPE}.ini"), EConfigLayerFlags::UseGlobalConfigCache },
	// Project/Platform/Platform*.ini
	{ TEXT("ProjectPlatform"),			TEXT("{PROJECT}/Config/{PLATFORM}/{PLATFORM}{TYPE}.ini"), EConfigLayerFlags::UseGlobalConfigCache },
	// Project/Platform/GeneratedPlatform*.ini Reserved for files generated by build process and should never be checked in 
	{ TEXT("ProjectPlatformGenerated"),	TEXT("{PROJECT}/Config/{PLATFORM}/Generated{PLATFORM}{TYPE}.ini"), EConfigLayerFlags::UseGlobalConfigCache },
	// Project/Platform/Custom/CustomConfig/Platform*.ini only if CustomConfig is defined
	{ TEXT("CustomConfigPlatform"),		TEXT("{PROJECT}/Config/{PLATFORM}/Custom/{CUSTOMCONFIG}/{PLATFORM}{TYPE}.ini"), EConfigLayerFlags::RequiresCustomConfig| EConfigLayerFlags::UseGlobalConfigCache },
	// AppSettings/.../System*.ini
	{ TEXT("AppSettingsDir"),			TEXT("{APPSETTINGS}Unreal Engine/Engine/Config/System{TYPE}.ini"), EConfigLayerFlags::NoExpand },
	// UserSettings/.../User*.ini
	{ TEXT("UserSettingsDir"),			TEXT("{USERSETTINGS}Unreal Engine/Engine/Config/User{TYPE}.ini"), EConfigLayerFlags::NoExpand },
	// UserDir/.../User*.ini
	{ TEXT("UserDir"),					TEXT("{USER}Unreal Engine/Engine/Config/User{TYPE}.ini"), EConfigLayerFlags::NoExpand },
	// Project/User*.ini
	{ TEXT("GameDirUser"),				TEXT("{PROJECT}/Config/User{TYPE}.ini"), EConfigLayerFlags::NoExpand },
};


/// <summary>
/// Plugins don't need to look at the same number of insane layers.
/// </summary>
inline FConfigLayer GPluginLayers[] =
{
	// Engine/Base.ini
	{ TEXT("AbsoluteBase"),				TEXT("{ENGINE}/Config/PluginBase.ini"), EConfigLayerFlags::NoExpand},

	// Plugin/Base*.ini (DEPRECATED - Use DefaultPluginName.ini instead!)
	// UE_DEPRECATED(5.8, "Use DefaultPluginName instead")
	{ TEXT("PluginBase"),				TEXT("{PLUGIN}/Config/Base{TYPE}.ini"), EConfigLayerFlags::UsePluginConfigCache | EConfigLayerFlags::Deprecated},
	// Plugin/Default*.ini
	{ TEXT("PluginDefault"),			TEXT("{PLUGIN}/Config/Default{TYPE}.ini"), EConfigLayerFlags::UsePluginConfigCache },
	// Plugin/*.ini (DEPRECATED - Use DefaultPluginName.ini instead!)
	// UE_DEPRECATED(5.8, "Use DefaultPluginName instead")
	{ TEXT("Plugin"),					TEXT("{PLUGIN}/Config/{TYPE}.ini"), EConfigLayerFlags::UsePluginConfigCache | EConfigLayerFlags::Deprecated },
	// Plugin/Platform/Platform*.ini
	{ TEXT("PluginPlatform"),			TEXT("{PLUGIN}/Config/{PLATFORM}/{PLATFORM}{TYPE}.ini"), EConfigLayerFlags::UsePluginConfigCache },
	// Project/Default.ini
	{ TEXT("ProjectDefault"),			TEXT("{PROJECT}/Config/Default{TYPE}.ini"), EConfigLayerFlags::UseGlobalConfigCache },
	// Project/Platform/Platform*.ini
	{ TEXT("ProjectDefault"),			TEXT("{PROJECT}/Config/{PLATFORM}/{PLATFORM}{TYPE}.ini"), EConfigLayerFlags::UseGlobalConfigCache },
};

/// <summary>
/// These are for the modifications to existing config files (for instance Plugin/Config/Engine.ini
/// </summary>
inline FConfigLayer GPluginModificationLayers[] =
{
	// Plugin/*.ini, can be plugin name, or say Engine.ini
	{ TEXT("PluginMod"),				TEXT("{PLUGIN}/Config/{TYPE}.ini") },
	// Plugin/Platform/Platform*.ini
	{ TEXT("PluginMModPlatform"),		TEXT("{PLUGIN}/Config/{PLATFORM}/{PLATFORM}{TYPE}.ini") },
};



/**************************************************
**** CRITICAL NOTES
**** If you change these arrays, you need to also change EnumerateConfigFileLocations() in ConfigHierarchy.cs!!!
**************************************************/
inline FConfigLayerExpansion GConfigExpansions[] =
{
	// No replacements
	{ nullptr, nullptr, nullptr, nullptr, EConfigExpansionFlags::All },

	// Restricted Locations
	{ 
		TEXT("{ENGINE}/"),						TEXT("{ENGINE}/Restricted/NotForLicensees/"),	
		TEXT("{PROJECT}/Config/"),				TEXT("{RESTRICTEDPROJECT_NFL}/Config/"),
		EConfigExpansionFlags::ForUncooked | EConfigExpansionFlags::ForCooked
	},
	{ 
		TEXT("{ENGINE}/"),						TEXT("{ENGINE}/Restricted/NoRedist/"),			
		TEXT("{PROJECT}/Config/"),				TEXT("{RESTRICTEDPROJECT_NR}/Config/"),
		EConfigExpansionFlags::ForUncooked 
	},
	{ 
		TEXT("{ENGINE}/"),						TEXT("{ENGINE}/Restricted/LimitedAccess/"),			
		TEXT("{PROJECT}/Config/"),				TEXT("{RESTRICTEDPROJECT_LA}/Config/"),
		EConfigExpansionFlags::ForUncooked | EConfigExpansionFlags::ForCooked
	},

	// Platform Extensions
	{
		TEXT("{ENGINE}/Config/{PLATFORM}/"),	TEXT("{EXTENGINE}/Config/"),
		TEXT("{PROJECT}/Config/{PLATFORM}/"),	TEXT("{EXTPROJECT}/Config/"),
		EConfigExpansionFlags::ForUncooked | EConfigExpansionFlags::ForCooked,
	},

	// Plugin Platform Extensions
	{
		TEXT("{PLUGIN}/Config/{PLATFORM}/"),	TEXT("{EXTPLUGIN}/Config/"),
		TEXT("{PROJECT}/Config/{PLATFORM}/"),	TEXT("{EXTPROJECT}/Config/"),
		EConfigExpansionFlags::ForPlugin,
	},

	// Platform Extensions in Restricted Locations
	// 
	// Regarding the commented EConfigExpansionFlags::ForPlugin expansions: in the interest of keeping plugin ini scanning fast,
	// we disable these expansions for plugins because they are not used by Epic, and are unlikely to be used by licensees. If
	// we can make scanning fast (caching what directories exist, etc), then we could turn this back on to be future-proof.
	{
		TEXT("{ENGINE}/Config/{PLATFORM}/"),	TEXT("{ENGINE}/Restricted/NotForLicensees/Platforms/{PLATFORM}/Config/"),	
		TEXT("{PROJECT}/Config/{PLATFORM}/"),	TEXT("{RESTRICTEDPROJECT_NFL}/Platforms/{PLATFORM}/Config/"), 
		EConfigExpansionFlags::ForUncooked | EConfigExpansionFlags::ForCooked // | EConfigExpansionFlags::ForPlugin 
	},
	{
		TEXT("{ENGINE}/Config/{PLATFORM}/"),	TEXT("{ENGINE}/Restricted/NoRedist/Platforms/{PLATFORM}/{OPT_SUBDIR}Config/"),
		TEXT("{PROJECT}/Config/{PLATFORM}/"),	TEXT("{RESTRICTEDPROJECT_NR}/Platforms/{PLATFORM}/{OPT_SUBDIR}Config/"), 
		EConfigExpansionFlags::ForUncooked // | EConfigExpansionFlags::ForPlugin
	},
	{
		TEXT("{ENGINE}/Config/{PLATFORM}/"),	TEXT("{ENGINE}/Restricted/LimitedAccess/Platforms/{PLATFORM}/{OPT_SUBDIR}Config/"),
		TEXT("{PROJECT}/Config/{PLATFORM}/"),	TEXT("{RESTRICTEDPROJECT_LA}/Platforms/{PLATFORM}/{OPT_SUBDIR}Config/"), 
		EConfigExpansionFlags::ForUncooked | EConfigExpansionFlags::ForCooked // | EConfigExpansionFlags::ForPlugin
	},
};


