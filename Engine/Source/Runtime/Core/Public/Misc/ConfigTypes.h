// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/Utf8String.h"
#include "Containers/UnrealString.h"
#include "Containers/Map.h"
#include "CoreGlobals.h"


class FConfigFileHierarchy : public TSparseMap<int32, FUtf8String>
{
public:
	FConfigFileHierarchy();

	friend FArchive& operator<<(FArchive& Ar, FConfigFileHierarchy& ConfigFileHierarchy)
	{
		Ar << static_cast<FConfigFileHierarchy::Super&>(ConfigFileHierarchy);
		return Ar;
	}

private:

	int32 AddStaticLayer(const FString& Filename, int32 LayerIndex, int32 ExpansionIndex, int32 PlatformIndex, int32 Flags);

	friend class FConfigFile;
	friend class FConfigContext;
};

struct FDynamicLayerInfo
{
	FString Filename;
	FName Tag;
	uint16 Priority;
};

enum class EConfigLayerFlags : int32
{
	None = 0,
	AllowCommandLineOverride = (1 << 1),
	DedicatedServerOnly = (1 << 2), // replaces Default, Base, and (NOT {PLATFORM} yet) with an empty string
	NoExpand = (1 << 4),
	RequiresCustomConfig = (1 << 5), // disabled if no custom config specified
	UseGlobalConfigCache = (1 << 6),
	UsePluginConfigCache = (1 << 7),
	Deprecated = (1 << 8), // allowed but will ensure or show some other deprecated message
};
ENUM_CLASS_FLAGS(EConfigLayerFlags);

/**
 * Structure to define all the layers of the config system. Layers can be expanded by expansion files (NoRedist, etc), or by ini platform parents
 */
struct FConfigLayer
{
	// Used by the editor to display in the ini-editor
	const TCHAR* EditorName;
	// Path to the ini file (with variables)
	const TCHAR* Path;
	// Special flag
	EConfigLayerFlags Flag;
	// Message to display if a deprecated file is found
	const FString DeprecatedMessage;

	// Will be cached early on to avoid threading issues
	uint8 bHasCheckedExist = false;
	uint8 bExists = false;

};

enum class EConfigExpansionFlags : int32
{
	None = 0,

	ForUncooked = 1 << 0,
	ForCooked = 1 << 1,
	ForPlugin = 1 << 2,
	
	ForEngine = 1 << 3,
	ForProject = 1 << 4,

	All = 0xFF,
};
ENUM_CLASS_FLAGS(EConfigExpansionFlags);


/**
 * This describes extra files per layer, to deal with restricted and NDA covered platform files that can't have the settings
 * be in the Base/Default ini files.
 * Note that we treat DedicatedServer as a "Platform" where it will have it's own directory of files, like a platform
 */
struct FConfigLayerExpansion
{
	// a set of replacements from the source file to possible other files
	const TCHAR* Before1;
	const TCHAR* After1;
	const TCHAR* Before2;
	const TCHAR* After2;
	EConfigExpansionFlags Flags;
};

