// Copyright Epic Games, Inc. All Rights Reserved.

/*-----------------------------------------------------------------------------
	Config cache.
-----------------------------------------------------------------------------*/

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Templates/Function.h"

class FString;
class IConsoleVariable;
class FConfigModificationTracker;


namespace UE::DynamicConfig
{
	extern CORE_API void PerformDynamicConfig(FName Tag, TFunction<void(class FConfigModificationTracker*)> PerformModification, TFunction<void(FConfigModificationTracker*)> PostModification=nullptr);

	// this isn't directly used in this module, but the OnlineHotfixManager and GameFeatures plugin use it to talk to each other
	extern CORE_API TMulticastDelegate<void(const FName& Tag, const FName& Branch, class FConfigModificationTracker* ModificationTracker)> HotfixPluginForBranch;

	// Symmetric teardown to HotfixPluginForBranch: fired once per plugin when a GFP is deactivated so hotfix
	// systems can remove per-plugin asset patches. Tag is the plugin name (FName of the plugin base filename).
	extern CORE_API TMulticastDelegate<void(const FName& Tag)> RemovePluginHotfixesFromDynamicBranch;

	// this calls the UObjectBaseUtility from code where object system is not linked (it also calls TSOnConfigSectionsChanged()!)
	extern CORE_API TMulticastDelegate<void(const FConfigModificationTracker* ChangeTracker)> ReloadObjects;

	extern CORE_API TMulticastDelegate<void(const FConfigModificationTracker* ChangeTracker)> UpdateCVarsAndDeviceProfiles;
}


namespace UE::ConfigUtilities
{
	/**
	 * Single function to set a cvar from ini (handing friendly names, cheats for shipping and message about cheats in non shipping)
	 */
	CORE_API void OnSetCVarFromIniEntry(const TCHAR* IniFile, const TCHAR* Key, const TCHAR* Value, uint32 SetBy, bool bAllowCheating=false, bool bNoLogging=false, FName Tag=NAME_None);

	/**
	 * When boot the game, use this function to apply cvars from last saved file from hotfix
	 */
	CORE_API void ApplyCVarsFromBootHotfix();
	
	/**
	 * Helper function to read the contents of an ini file and a specified group of cvar parameters, where sections in the ini file are marked [InName]
	 * @param InSectionBaseName - The base name of the section to apply cvars from
	 * @param InIniFilename - The ini filename
	 * @param SetBy anything in ECVF_LastSetMask e.g. ECVF_SetByScalability
	 */
	CORE_API void ApplyCVarSettingsFromIni(const TCHAR* InSectionBaseName, const TCHAR* InIniFilename, uint32 SetBy, bool bAllowCheating=false, FName Tag=NAME_None);


	CORE_API void SaveCVarForNextBoot(const TCHAR* Key, const TCHAR* Value);

#if UE_EDITOR
	CORE_API void ApplyCVarOfDllFromBootHotfix(const TCHAR* Key, IConsoleObject* Obj);
#endif

	/**
	 * Helper function to operate a user defined function for each CVar key/value pair in the specified section in an ini file
	 * @param InSectionName - The name of the section to apply cvars from
	 * @param InIniFilename - The ini filename
	 * @param InEvaluationFunction - The evaluation function to be called for each key/value pair
	 */
	CORE_API void ForEachCVarInSectionFromIni(const TCHAR* InSectionName, const TCHAR* InIniFilename, TFunction<void(IConsoleVariable* CVar, const FString& KeyString, const FString& ValueString)> InEvaluationFunction);

	/**
	 * CVAR Ini history records all calls to ApplyCVarSettingsFromIni and can re run them
	 */

	 /**
	  * Helper function to start recording ApplyCVarSettings function calls
	  * uses these to generate a history of applied ini settings sections
	  */
	CORE_API void RecordApplyCVarSettingsFromIni();

	/**
	 * Helper function to reapply inis which have been applied after RecordCVarIniHistory was called
	 */
	CORE_API void ReapplyRecordedCVarSettingsFromIni();

	/**
	 * Helper function to clean up ini history
	 */
	CORE_API void DeleteRecordedCVarSettingsFromIni();

	/**
	 * Helper function to start recording config reads
	 */
	CORE_API void RecordConfigReadsFromIni();

	/**
	 * Helper function to dump config reads to csv after RecordConfigReadsFromIni was called
	 */
	CORE_API void DumpRecordedConfigReadsFromIni();

	/**
	 * Helper function to clean up config read history
	 */
	CORE_API void DeleteRecordedConfigReadsFromIni();

	/**
	 * Helper function to deal with "True","False","Yes","No","On","Off"
	 */
	CORE_API const TCHAR* ConvertValueFromHumanFriendlyValue(const TCHAR* Value);
}


