// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"
#include "Internationalization/Text.h"
#include "Misc/AsyncTaskNotification.h"
#include "Misc/Attribute.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "UObject/NameTypes.h"

struct FLogCategoryBase;

/**
 * Implementation interface of notifications for an on-going asynchronous task.
 */
class IAsyncTaskNotificationImpl
{
public:
	virtual ~IAsyncTaskNotificationImpl() = default;

	/**
	 * Initialize this notification based on the given config.
	 */
	virtual void Initialize(const FAsyncTaskNotificationConfig& InConfig) = 0;

	/**
	 * Set the title text of this notification.
	 */
	virtual void SetTitleText(const FText& InTitleText, const bool bClearProgressText) = 0;

	/**
	 * Set the progress text of this notification.
	 */
	virtual void SetProgressText(const FText& InProgressText) = 0;

	/**
	 * Set the prompt text of this notification.
	 */
	virtual void SetPromptText(const FText& InPromptText) = 0;

	/**
	 * Set the hyperlink text of this notification.
	 */
	virtual void SetHyperlink(const FSimpleDelegate& InHyperlink, const FText& InHyperlinkText) = 0;

	/**
	 * Set the task as complete.
	 */
	virtual void SetComplete(const bool bSuccess) = 0;

	/**
	 * Update the text and set the task as complete.
	 */
	virtual void SetComplete(const FText& InTitleText, const FText& InProgressText, const bool bSuccess) = 0;

	/**
	 * Set the task notification state. provides finer control than SetComplete
	 */
	virtual void SetNotificationState(const FAsyncNotificationStateData& InState) = 0;

	/**
	 * Set whether this task be canceled.
	 */
	virtual void SetCanCancel(const TAttribute<bool>& InCanCancel) = 0;

	/**
	 * Set whether to keep this notification open on success.
	*/
	virtual void SetKeepOpenOnSuccess(const TAttribute<bool>& InKeepOpenOnSuccess) = 0;

	/**
	 * Set whether to keep this notification open on failure.
	 */
	virtual void SetKeepOpenOnFailure(const TAttribute<bool>& InKeepOpenOnFailure) = 0;

	/**
	 * Return the notification prompt action.
	 * The action resets to `None` when the notification state changes.
	 */
	virtual EAsyncTaskNotificationPromptAction GetPromptAction() const = 0;
};

/**
 * Basic asynchronous task notification that just logs status changes.
 */
class FCoreAsyncTaskNotificationImpl : public IAsyncTaskNotificationImpl
{
public:
	CORE_API FCoreAsyncTaskNotificationImpl();
	CORE_API virtual ~FCoreAsyncTaskNotificationImpl();

	//~ IAsyncTaskNotificationImpl
	CORE_API virtual void Initialize(const FAsyncTaskNotificationConfig& InConfig) override;
	CORE_API virtual void SetTitleText(const FText& InTitleText, const bool bClearProgressText) override;
	CORE_API virtual void SetProgressText(const FText& InProgressText) override;
	CORE_API virtual void SetPromptText(const FText& InPromptText) override;
	CORE_API virtual void SetHyperlink(const FSimpleDelegate& InHyperlink, const FText& InHyperlinkText) override;
	CORE_API virtual void SetComplete(const bool bSuccess) override;
	CORE_API virtual void SetComplete(const FText& InTitleText, const FText& InProgressText, const bool bSuccess) override;
	CORE_API virtual void SetNotificationState(const FAsyncNotificationStateData& InState) override;
	CORE_API virtual void SetCanCancel(const TAttribute<bool>& InCanCancel) override;
	CORE_API virtual void SetKeepOpenOnSuccess(const TAttribute<bool>& InKeepOpenOnSuccess) override;
	CORE_API virtual void SetKeepOpenOnFailure(const TAttribute<bool>& InKeepOpenOnFailure) override;
	CORE_API virtual EAsyncTaskNotificationPromptAction GetPromptAction() const override;

protected:
	/** Update the notification (the critical section is held while this function is called) */
	CORE_API virtual void UpdateNotification();

	/** The current state of this notification */
	EAsyncTaskNotificationState State = EAsyncTaskNotificationState::Pending;

	/** The title text displayed in the notification (if any) */
	FText TitleText;

	/** The progress text displayed in the notification (if any) */
	FText ProgressText;

	/** The text used by the prompt button (if any) */
	FText PromptText;

	/** When set this will display as a hyperlink on the right side of the notification. */
	FSimpleDelegate Hyperlink;

	/** Text to display for the hyperlink message */
	FText HyperlinkText;
private:
	/** Log the current notification state (if any, and if enabled) */
	void LogNotification();

	/** Category this task should log its notifications under, or null to skip logging */
	const FLogCategoryBase* LogCategory = nullptr;

	/** Critical section protecting concurrent access to this object state */
	mutable FCriticalSection SynchronizationObject;
};

/**
 * Factory to allow other systems (such as Slate) to override the default asynchronous task notification implementation.
 */
class FAsyncTaskNotificationFactory
{
	friend class FAsyncTaskNotification;

public:
	typedef TSharedPtr<IAsyncTaskNotificationImpl> FImplPointerType;
	typedef TFunction<FImplPointerType()> FFactoryFunc;

	/**
	 * Get the factory singleton.
	 */
	static CORE_API FAsyncTaskNotificationFactory& Get();

	/**
	 * Register a factory function.
	 */
	CORE_API void RegisterFactory(const FName InName, const FFactoryFunc& InFunc);

	/**
	 * Unregister a factory function.
	 */
	CORE_API void UnregisterFactory(const FName InName);

private:
	/**
	 * Invoke the active factory function (if any), or return a default instance.
	 */
	CORE_API FImplPointerType InvokeFactory() const;

	/** Registered factories */
	TArray<TTuple<FName, FFactoryFunc>> Factories;
};
