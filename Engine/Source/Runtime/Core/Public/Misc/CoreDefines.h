// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#ifndef STUBBED
#define STUBBED(x)	\
	do																								\
	{																								\
		static bool AlreadySeenThisStubbedSection = false;											\
		if (!AlreadySeenThisStubbedSection)															\
		{																							\
			AlreadySeenThisStubbedSection = true;													\
			fprintf(stderr, "STUBBED: %s at %s:%d (%s)\n", x, __FILE__, __LINE__, __FUNCTION__);	\
		}																							\
	} while (0)
#endif

/*----------------------------------------------------------------------------
Metadata macros.
----------------------------------------------------------------------------*/

#define CPP       1
#define STRUCTCPP 1
#define DEFAULTS  0


/*-----------------------------------------------------------------------------
Seek-free defines.
-----------------------------------------------------------------------------*/

#define STANDALONE_SEEKFREE_SUFFIX	TEXT("_SF")


/*-----------------------------------------------------------------------------
Macros for enabling heap storage instead of inline storage on delegate types.
Can be overridden by setting to 1 or 0 in the project's .Target.cs files.
-----------------------------------------------------------------------------*/

#ifndef NUM_DELEGATE_INLINE_BYTES
	#define NUM_DELEGATE_INLINE_BYTES 0
#endif

#ifndef NUM_MULTICAST_DELEGATE_INLINE_ENTRIES
	#define NUM_MULTICAST_DELEGATE_INLINE_ENTRIES 0
#endif

#ifndef NUM_TFUNCTION_INLINE_BYTES
	#define NUM_TFUNCTION_INLINE_BYTES 24
#endif

#ifndef NO_CVARS
	#define NO_CVARS 0
#endif

#ifndef UE_DEPRECATE_MUTABLE_TOBJECTPTR
#define UE_DEPRECATE_MUTABLE_TOBJECTPTR 1
#endif

#ifndef UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR
#define UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR 0
#endif

#if UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR
#undef UE_DEPRECATE_MUTABLE_TOBJECTPTR
#define UE_DEPRECATE_MUTABLE_TOBJECTPTR 1
#endif
