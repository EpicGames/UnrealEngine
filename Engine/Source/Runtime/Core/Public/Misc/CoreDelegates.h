// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphFwd.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformMisc.h"
#include "Logging/LogCategory.h"
#include "Logging/LogVerbosity.h"
#include "Math/IntVector.h"
#include "Math/MathFwd.h"
#include "Misc/AES.h"
#include "Misc/Build.h"
#include "Misc/Optional.h"
#include "IO/IoStatus.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

#ifndef USE_MMAPPED_SHADERARCHIVE
	// If enabled FSerializedShaderArchive, FIoStoreShaderCodeArchive and FMetalShaderLibrary will use a mmapping of the on-disk shader archive instead of normal serialization.
	// The on-disk format must match the expected in memory layout.
	#define USE_MMAPPED_SHADERARCHIVE 0
#endif

class AActor;
class Error;
class FConfigFile;
class FName;
class FText;
class IPakFile;
class ITargetPlatform;
struct FGuid;
struct FShaderHash;

enum class EForkProcessRole : uint8;

// delegates for hotfixes
namespace EHotfixDelegates
{
	enum Type
	{
		Test,
	};
}


// this is an example of a hotfix arg and return value structure. Once we have other examples, it can be deleted.
struct FTestHotFixPayload
{
	FString Message;
	bool ValueToReturn;
	bool Result;
};

// Parameters passed to CrashOverrideParamsChanged used to customize crash report client behavior/appearance. If the corresponding bool is not true, this value will not be stored.
struct FCrashOverrideParameters
{
	/** Appended to the end of GameName (which is retreived from FApp::GetGameName). */
	FString GameNameSuffix;
	/** Default this to true for backward compatibility before these bools were added. */
	bool bSetCrashReportClientMessageText = true;
	bool bSetGameNameSuffix = false;
	TOptional<bool> SendUnattendedBugReports;
	TOptional<bool> SendUsageData;

	CORE_API ~FCrashOverrideParameters();
};

namespace UE
{
struct FMultiprocessCreatedContext
{
	int32 Id;
};

struct FMultiprocessDetachedContext
{
	int32 Id;
	bool bAbnormalDetach;
};

struct FMountPaksExArgs;

}

class FCoreDelegates
{
public:
	// Callback for platform handling when flushing async loads.
	static CORE_API TMulticastDelegate<void()> OnAsyncLoadingFlush;

	// Callback for a game thread interruption point when a async load flushing. Used to updating UI during long loads.
	static CORE_API TMulticastDelegate<void()> OnAsyncLoadingFlushUpdate;

	// Thread-safe callback that is called on the same thread that LoadPackageAsync is issued from.
	static CORE_API TTSMulticastDelegate<void(FStringView)>& GetOnAsyncLoadPackage();

	static CORE_API TMulticastDelegate<void(const FString&)> OnSyncLoadPackage;

	// get a hotfix delegate
	static CORE_API TDelegate<void(void*, int32)>& GetHotfixDelegate(EHotfixDelegates::Type HotFix);

	// Callback when a user logs in/out of the platform.
	static CORE_API TMulticastDelegate<void(bool, int32, int32)> OnUserLoginChangedEvent;

	// Callback when a user changes the safe frame size
	static CORE_API TMulticastDelegate<void()> OnSafeFrameChangedEvent;

	// Callback for mounting all the pak files in default locations
	static CORE_API TDelegate<int32(const TArray<FString>&)> OnMountAllPakFiles;

	// Callback to prompt the pak system to mount a pak file
	static CORE_API TDelegate<IPakFile*(const FString&, int32)> MountPak;

	// Callback to prompt the pak system to mount multiple pak files
	// Returns true if any paks were mounted.
	static CORE_API TDelegate<bool(TArrayView<UE::FMountPaksExArgs>)> MountPaksEx;

	// Callback to prompt the pak system to unmount a pak file.
	static CORE_API TDelegate<bool(const FString&)> OnUnmountPak;

	// Used by asset manager to translate from a pakchunk name to the chunkId it should have.
	static CORE_API TDelegate<int32(const FString& /*InPakchunkName*/)> PakChunkIdOverrideDelegate;

	// Callback to optimize memeory for currently mounted paks
	static CORE_API TDelegate<void()> OnOptimizeMemoryUsageForMountedPaks;

	// After a pakfile is mounted this is called
	static CORE_API TTSMulticastDelegate<void(const IPakFile&)>& GetOnPakFileMounted2();

	// Before a pakfile is unmounted this is called
	static CORE_API TTSMulticastDelegate<void(const IPakFile&)>& GetOnPakFileUnmounting();

	// After a pakfile is unmounted this is called
	static CORE_API TTSMulticastDelegate<void(const IPakFile&)>& GetOnPakFileUnmounted();

	// After a file is added this is called
	static CORE_API TMulticastDelegate<void(const FString&)> NewFileAddedDelegate;

	// After an attempt to mount all pak files, but none wre found, this is called
	static CORE_API TMulticastDelegate<void()> NoPakFilesMountedDelegate;

	// Called when a pak file cannot be mounted
	static CORE_API TMulticastDelegate<void(const FString&)> PakFileMountFailedDelegate;

	// When a file is opened for read from a pak file.
	static CORE_API TTSMulticastDelegate<void(const TCHAR* PakFile, const TCHAR* FileName)>& GetOnFileOpenedForReadFromPakFile();

	typedef TSharedPtr<class IMovieStreamer, ESPMode::ThreadSafe> FMovieStreamerPtr;

    // Delegate used to register a movie streamer with any movie player modules that bind to this delegate
    // Designed to be called when a platform specific movie streamer plugin starts up so that it doesn't need to implement a register for all movie player plugins
    static CORE_API TMulticastDelegate<void(FMovieStreamerPtr)> RegisterMovieStreamerDelegate;
    // Delegate used to un-register a movie streamer with any movie player modules that bind to this delegate
    // Designed to be called when a platform specific movie streamer plugin shuts down so that it doesn't need to implement a register for all movie player plugins
    static CORE_API TMulticastDelegate<void(FMovieStreamerPtr)> UnRegisterMovieStreamerDelegate;

	// Callback when an ensure has occurred
	static CORE_API TMulticastDelegate<void(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, const TCHAR* Msg, const TCHAR* CombinedErrorMsg)> OnEnsureFailed;

	// Callback when an ensure error report is being submitted
	static CORE_API TMulticastDelegate<void()> OnHandleSystemEnsure;

	// Callback when an error (crash) has occurred
	static CORE_API TMulticastDelegate<void()> OnHandleSystemError;

	// Callback when a fatal hang has occurred.
	// WARNING: Use AddRaw/AddStatic for this delegate. Other bindings could be unsafe and cause deadlocks, e.g. AddUObject invoked during GC.
	static CORE_API TMulticastDelegate<void()> OnHandleSystemHang;

	// Called when an actor label is changed
	static CORE_API TMulticastDelegate<void(AActor*)> OnActorLabelChanged;

	static CORE_API TMulticastDelegate<void(const FGuid&, const FAES::FAESKey&)>& GetRegisterEncryptionKeyMulticastDelegate();
	static CORE_API TDelegate<void(uint8[32])>& GetPakEncryptionKeyDelegate();
	static CORE_API TDelegate<void(TArray<uint8>&, TArray<uint8>&)>& GetPakSigningKeysDelegate();

	

#if WITH_EDITOR
	struct FModalWindowContext
	{
		// Only available in PreSlateModalWithContext
		TOptional<bool> bIsSlowTaskWindow;

		// Guaranteed to match between PreSlateModalWithContext and PostSlateModalWithContext
		typedef std::intptr_t FWindowHandle;
		FWindowHandle WindowIdentifier = 0;
	};

	DECLARE_MULTICAST_DELEGATE_OneParam(FPrePostSlateModalWithContextDelegate, const FModalWindowContext&);

	// Called before the editor displays a modal window, allowing other windows the opportunity to disable themselves to avoid reentrant calls
	static CORE_API FSimpleMulticastDelegate PreModal;

	// Called after the editor dismisses a modal window, allowing other windows the opportunity to disable themselves to avoid reentrant calls
	static CORE_API FSimpleMulticastDelegate PostModal;
    
	UE_DEPRECATED(5.8, "PreSlateModal has been deprecated. Please use PreSlateModalWithContext instead.")
    static CORE_API FSimpleMulticastDelegate PreSlateModal;
    // Called before the editor displays a Slate (non-platform) modal window, allowing other windows the opportunity to disable themselves to avoid reentrant calls
	static CORE_API FPrePostSlateModalWithContextDelegate PreSlateModalWithContext;
    
    // Called before the editor displays a Slate (non-platform) modal window, allowing other windows the opportunity to disable themselves to avoid reentrant calls
	UE_DEPRECATED(5.8, "PostSlateModal has been deprecated. Please use PostSlateModalWithContext instead.")
    static CORE_API FSimpleMulticastDelegate PostSlateModal;

    // Called after the editor dismisses a Slate (non-platform) modal window, allowing other windows the opportunity to disable themselves to avoid reentrant calls
	static CORE_API FPrePostSlateModalWithContextDelegate PostSlateModalWithContext;
    
#endif	//WITH_EDITOR

#if ALLOW_OTHER_PLATFORM_CONFIG
	// Called when the CVar (ConsoleManager) needs to retrieve CVars for a deviceprofile for another platform - this dramatically simplifies module dependencies
	typedef TMap<FName, FString> FCVarKeyValueMap;
	static CORE_API TDelegate<FCVarKeyValueMap(const FString& DeviceProfileName)> GatherDeviceProfileCVars;
#endif

	// Called when an error occurred.
	static CORE_API FSimpleMulticastDelegate OnShutdownAfterError;

	// Called when appInit is called, very early in startup
	static CORE_API FSimpleMulticastDelegate OnInit;

	// Called during FEngineLoop::PreInit after GWarn & GError have been first set so that they can be overridden before anything in PreInit uses them
	static CORE_API FSimpleMulticastDelegate OnOutputDevicesInit;

	// Called at the end of UEngine::Init, right before loading PostEngineInit modules for both normal execution and commandlets
	static CORE_API FSimpleMulticastDelegate& GetOnPostEngineInit();

	// Called at the end of UEngine::Init, right before loading PostEngineInit modules for both normal execution and commandlets
	UE_DEPRECATED(5.8, "OnPostEngineInit has been deprecated. Please use GetOnPostEngineInit() instead.")
	static CORE_API FSimpleMulticastDelegate OnPostEngineInit;

	// Called after all modules have been loaded for all phases
	static CORE_API FSimpleMulticastDelegate OnAllModuleLoadingPhasesComplete;

	// Called at the very end of engine initialization, right before the engine starts ticking. This is not called for commandlets
	static CORE_API FSimpleMulticastDelegate OnFEngineLoopInitComplete;

	// Called prior to running the Main function for commandlets
	static CORE_API FSimpleMulticastDelegate OnCommandletPreMain;

	// Called after running the Main function for commandlets
	static CORE_API FSimpleMulticastDelegate OnCommandletPostMain;

	// Called when the application is about to exit.
	static CORE_API FSimpleMulticastDelegate OnExit;

	// Called when before the application is exiting.
	static CORE_API FSimpleMulticastDelegate OnPreExit;

	// Called before the engine exits. Separate from OnPreExit as OnEnginePreExit occurs before shutting down any core modules.
	static CORE_API FSimpleMulticastDelegate OnEnginePreExit;

	/** Delegate for gathering up additional localization paths that are unknown to the engine core (such as plugins) */
	static CORE_API TMulticastDelegate<void(TArray<FString>&)> GatherAdditionalLocResPathsCallback;

	/** Color picker color has changed, please refresh as needed*/
	static CORE_API FSimpleMulticastDelegate ColorPickerChanged;

	/** requests to open a message box */
	static CORE_API TDelegate<EAppReturnType::Type(EAppMsgCategory, EAppMsgType::Type, const FText&, const FText&)> ModalMessageDialog;

	/** Called when the user accepts an invitation to the current game */
	static CORE_API TMulticastDelegate<void(const FString&, const FString&)> OnInviteAccepted;

	// Called at the beginning of a frame
	static CORE_API FSimpleMulticastDelegate OnBeginFrame;

	// Called at the moment of sampling the input (currently on the gamethread)
	static CORE_API FSimpleMulticastDelegate OnSamplingInput;

	// Called at the end of a frame
	static CORE_API FSimpleMulticastDelegate OnEndFrame;

	// Called at the beginning of a frame on the renderthread
	static CORE_API FSimpleMulticastDelegate OnBeginFrameRT;

	// Called at the end of a frame on the renderthread
	static CORE_API FSimpleMulticastDelegate OnEndFrameRT;


	/** called before world origin shifting */
	static CORE_API TMulticastDelegate<void(class UWorld*, FIntVector, FIntVector)> PreWorldOriginOffset;
	/** called after world origin shifting */
	static CORE_API TMulticastDelegate<void(class UWorld*, FIntVector, FIntVector)> PostWorldOriginOffset;

	/** called when the main loop would otherwise starve. */
	static CORE_API TDelegate<void()> StarvedGameLoop;

	// IOS-style temperature updates, allowing game to scale down to let temp drop (to avoid thermal throttling on mobile, for instance) */
	// There is a parellel enum in ApplicationLifecycleComponent
	enum class ETemperatureSeverity : uint8
	{
		Unknown,
		Good,
		Bad,
		Serious,
		Critical,

		NumSeverities,
	};
	static CORE_API TMulticastDelegate<void(ETemperatureSeverity)> OnTemperatureChange;

	/** Called when the OS goes into low power mode */
	static CORE_API TMulticastDelegate<void(bool)> OnLowPowerMode;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNetworkConnectionChanged, ENetworkConnectionType /*ConnectionType*/);
	static CORE_API FOnNetworkConnectionChanged OnNetworkConnectionChanged;

	static CORE_API TTSMulticastDelegate<void(const TCHAR* IniFilename, int32& ResponderCount)>& TSCountPreLoadConfigFileRespondersDelegate();
	static CORE_API TTSMulticastDelegate<void(const TCHAR* IniFilename, FString& LoadedContents)>& TSPreLoadConfigFileDelegate();
	static CORE_API TTSMulticastDelegate<void(const TCHAR* IniFilename, const FString& ContentsToSave, int32& SavedCount)>& TSPreSaveConfigFileDelegate();
	static CORE_API TTSMulticastDelegate<void(const TCHAR* IniFilename, bool& bOutDeleted)>& TSPreDeleteConfigFileDelegate();
	static CORE_API TTSMulticastDelegate<void(const FConfigFile*)>& TSOnFConfigCreated();
	static CORE_API TTSMulticastDelegate<void(const FConfigFile*)>& TSOnFConfigDeleted();
	static CORE_API TTSMulticastDelegate<void(const TCHAR* IniFilename, const TCHAR* SectionName, const TCHAR* Key)>& TSOnConfigValueRead();
	static CORE_API TTSMulticastDelegate<void(const TCHAR* IniFilename, const TCHAR* SectionName)>& TSOnConfigSectionRead();
	static CORE_API TTSMulticastDelegate<void(const TCHAR* IniFilename, const TCHAR* SectionName)>& TSOnConfigSectionNameRead();
	static CORE_API TTSMulticastDelegate<void(const FString& IniFilename, const TSet<FString>& SectionNames)>& TSOnConfigSectionsChanged();
	static CORE_API TTSMulticastDelegate<void(const TCHAR* IniFilename)>& TSOnConfigBranchUnloaded();
	static CORE_API TTSMulticastDelegate<void(const TCHAR* IniFilename, float ReloadTimeMs)>& TSOnConfigBranchReloaded();

	static CORE_API TMulticastDelegate<void(const TCHAR* SectionName, const TCHAR* IniFilename, uint32 SetBy, bool bAllowCheating)> OnApplyCVarFromIni;

	static CORE_API TMulticastDelegate<void(uint32 ResX, uint32 ResY)> OnSystemResolutionChanged;

#if WITH_EDITOR
	// called when a target platform changes it's return value of supported formats.  This is so anything caching those results can reset (like cached shaders for cooking)
	static CORE_API TMulticastDelegate<void(const ITargetPlatform*)> OnTargetPlatformChangedSupportedFormats;

	// Called when a feature level is disabled by the user.
	static CORE_API TMulticastDelegate<void(int, const FName&)> OnFeatureLevelDisabled;
#endif

	/** IOS-style application lifecycle delegates */

	// This is called when the application is about to be deactivated (e.g., due to a phone call or SMS or the sleep button).
	// The game should be paused if possible, etc...
	static CORE_API TMulticastDelegate<void()> ApplicationWillDeactivateDelegate;

	// Called when the application has been reactivated (reverse any processing done in the Deactivate delegate)
	static CORE_API TMulticastDelegate<void()> ApplicationHasReactivatedDelegate;

	// This is called when the application is being backgrounded (e.g., due to switching
	// to another app or closing it via the home button)
	// The game should release shared resources, save state, etc..., since it can be
	// terminated from the background state without any further warning.
	static CORE_API TMulticastDelegate<void()> ApplicationWillEnterBackgroundDelegate; // for instance, hitting the home button

	// Called when the application is returning to the foreground (reverse any processing done in the EnterBackground delegate)
	static CORE_API TMulticastDelegate<void()> ApplicationHasEnteredForegroundDelegate;

	// This *may* be called when the application is getting terminated by the OS.
	// There is no guarantee that this will ever be called on a mobile device,
	// save state when ApplicationWillEnterBackgroundDelegate is called instead.
	static CORE_API TTSMulticastDelegate<void()>& GetApplicationWillTerminateDelegate();

	// Some platform have a System UI Overlay that can draw on top of the application.
	// The game might want to be notified so it can pause, etc...
	// Parameter (bool) should be true if the system UI is displayed, otherwise false should be passed
	static CORE_API TMulticastDelegate<void(bool)> ApplicationSystemUIOverlayStateChangedDelegate;

	// Called when in the background, if the OS is giving CPU time to the device. It is very likely
	// this will never be called due to mobile OS backgrounded CPU restrictions. But if, for instance,
	// VOIP is active on iOS, the will be getting called
	static CORE_API TMulticastDelegate<void(float DeltaTime)> MobileBackgroundTickDelegate;

	// Called when the OS needs control of the music (parameter is true) or when the OS returns
	// control of the music to the application (parameter is false). This can happen due to a
	// phone call or timer or other OS-level event. This is currently triggered only on iOS
	// devices.
	static CORE_API TMulticastDelegate<void(bool)> UserMusicInterruptDelegate;
	
	// [iOS only] Called when the mute switch is detected as changed or when the
	// volume changes. Parameter 1 is the mute switch state (true is muted, false is
	// unmuted). Parameter 2 is the volume as an integer from 0 to 100.
	static CORE_API TMulticastDelegate<void(bool, int)> AudioMuteDelegate;
	
	// [iOS only] Called when the audio device changes
	// For instance, when the headphones are plugged in or removed
	static CORE_API TMulticastDelegate<void(bool)> AudioRouteChangedDelegate;

	// [iOS only] Called when the audio session is interrupted
	static CORE_API TMulticastDelegate<void(bool)> AudioInterruptionDelegate;
	
	// Generally, events triggering UserMusicInterruptDelegate or AudioMuteDelegate happen only
	// when a change occurs. When a system comes online needing the current audio state but the
	// event has already been broadcast, calling ApplicationRequestAudioState will force the
	// UserMusicInterruptDelegate and AudioMuteDelegate to be called again if the low-level
	// application layer supports it. Currently, this is available only on iOS.
	static CORE_API TMulticastDelegate<void()> ApplicationRequestAudioState;
	
	// Called when the OS is running low on resources and asks the application to free up any cached resources, drop graphics quality etc.
	static CORE_API TMulticastDelegate<void()> ApplicationShouldUnloadResourcesDelegate;

	// Called with arguments passed to the application on statup, perhaps meta data passed on by another application which launched this one.
	static CORE_API TMulticastDelegate<void(const TArray<FString>&)> ApplicationReceivedStartupArgumentsDelegate;

	/** IOS-style push notification delegates */

	// called when the user grants permission to register for remote notifications
	static CORE_API TMulticastDelegate<void(TArray<uint8>)> ApplicationRegisteredForRemoteNotificationsDelegate;

	// called when the user grants permission to register for notifications
	static CORE_API TMulticastDelegate<void(int)> ApplicationRegisteredForUserNotificationsDelegate;

	// called when the application fails to register for remote notifications
	static CORE_API TMulticastDelegate<void(FString)> ApplicationFailedToRegisterForRemoteNotificationsDelegate;

	// called when the application receives a remote notification
	static CORE_API TMulticastDelegate<void(FString, int)> ApplicationReceivedRemoteNotificationDelegate;

	// called when the application receives a local notification
	static CORE_API TMulticastDelegate<void(FString, int, int)> ApplicationReceivedLocalNotificationDelegate;

    // called when the application receives notice to perform a background fetch
    static CORE_API TMulticastDelegate<void()> ApplicationPerformFetchDelegate;

    // called when the application receives notice that a background download has completed
    static CORE_API TMulticastDelegate<void(FString)> ApplicationBackgroundSessionEventDelegate;

	/** Sent when a device screen orientation changes */
	static CORE_API TMulticastDelegate<void(int32)> ApplicationReceivedScreenOrientationChangedNotificationDelegate;

	/** Checks to see if the stat is already enabled */
	static CORE_API TMulticastDelegate<void(const TCHAR*, bool&, bool&)> StatCheckEnabled;

	/** Sent after each stat is enabled */
	static CORE_API TMulticastDelegate<void(const TCHAR*)> StatEnabled;

	/** Sent after each stat is disabled */
	static CORE_API TMulticastDelegate<void(const TCHAR*)> StatDisabled;

	/** Sent when all stats need to be disabled */
	static CORE_API TMulticastDelegate<void(const bool)> StatDisableAll;

	// Called when an application is notified that the application license info has been updated.
	// The new license data should be polled and steps taken based on the results (i.e. halt application if license is no longer valid).
	static CORE_API TMulticastDelegate<void()> ApplicationLicenseChange;
	
	// Called periodically from within slow operations in order to indicate the application is still alive.
	// Progress must be made for this to be called, or it could prevent the application from detecting soft-locks and unresponsiveness.
	// Subscribers should be lightweight and thread-safe.
	static CORE_API TTSMulticastDelegate<void()> ApplicationHeartbeat;

	/** Sent when the platform changed its laptop mode (for convertible laptops).*/
	static CORE_API TMulticastDelegate<void(EConvertibleLaptopMode)> PlatformChangedLaptopMode;

	/** Sent when the platform needs the user to fix headset tracking on startup (Most platforms do not need this.) */
	static CORE_API TMulticastDelegate<void()> VRHeadsetTrackingInitializingAndNeedsHMDToBeTrackedDelegate;

	/** Sent when the platform finds that needed headset tracking on startup has completed (Most platforms do not need this.) */
	static CORE_API TMulticastDelegate<void()> VRHeadsetTrackingInitializedDelegate;

	/** Sent when the platform requests a low-level VR recentering */
	static CORE_API TMulticastDelegate<void()> VRHeadsetRecenter;

	/** Sent when connection to VR HMD is lost */
	static CORE_API TMulticastDelegate<void()> VRHeadsetLost;

	/** Sent when connection to VR HMD is restored */
	static CORE_API TMulticastDelegate<void()> VRHeadsetReconnected;

	/** Sent when connection to VR HMD connection is refused by the player */
	static CORE_API TMulticastDelegate<void()> VRHeadsetConnectCanceled;

	/** Sent when the VR HMD detects that it has been put on by the player. */
	static CORE_API TMulticastDelegate<void()> VRHeadsetPutOnHead;

	/** Sent when the VR HMD detects that it has been taken off by the player. */
	static CORE_API TMulticastDelegate<void()> VRHeadsetRemovedFromHead;

	/** Sent when a 3DOF VR controller is recentered */
	static CORE_API TMulticastDelegate<void()> VRControllerRecentered;

	/** Sent when application code changes the user activity hint string for analytics, crash reports, etc */
	static CORE_API TMulticastDelegate<void(const FString&)> UserActivityStringChanged;

	/** Sent when application code changes the currently active game session. The exact semantics of this will vary between games but it is useful for analytics, crash reports, etc  */
	static CORE_API TMulticastDelegate<void(const FString&)> GameSessionIDChanged;

	/** Sent when application code changes game state. The exact semantics of this will vary between games but it is useful for analytics, crash reports, etc  */
	static CORE_API TMulticastDelegate<void(const FString&)> GameStateClassChanged;

	/** Sent by application code to set params that customize crash reporting behavior. */
	static CORE_API TMulticastDelegate<void(const FCrashOverrideParameters&)> CrashOverrideParamsChanged;
	
	/** Sent by engine code when the "vanilla" status of the engine changes */
	static CORE_API TMulticastDelegate<void(bool)> IsVanillaProductChanged;

	// Callback for platform specific very early init code.
	static CORE_API TMulticastDelegate<void()>& GetPreMainInitDelegate();
	
	/** Sent when GConfig is finished initializing */
	static CORE_API TTSMulticastDelegate<void()>& TSConfigReadyForUse();

	/** Callback for notifications regarding changes of the rendering thread. */

	/** Sent just after the rendering thread has been created. */
	static CORE_API TMulticastDelegate<void()> PostRenderingThreadCreated;
	/* Sent just before the rendering thread is destroyed. */
	static CORE_API TMulticastDelegate<void()> PreRenderingThreadDestroyed;

	// Callback to allow custom resolution of package names. Arguments are InRequestedName, OutResolvedName.
	// Should return True of resolution occured.
	static CORE_API TArray<TDelegate<bool(const FString&, FString&)>> PackageNameResolvers;

	/** Callback to allow for rare specialized situations where manual clearing of references is necessary */
	static CORE_API TMulticastDelegate<void(const TArrayView<UObject*>)> CleanupUnloadingObjects;

	// Called to request that systems free whatever memory they are able to. Called early in LoadMap.
	// Caller is responsible for flushing rendering etc. See UEngine::TrimMemory
	static CORE_API FSimpleMulticastDelegate& GetMemoryTrimDelegate();

	// Called to request that low level allocator free whatever memory they are able to. 
	static CORE_API FSimpleMulticastDelegate& GetLowLevelAllocatorMemoryTrimDelegate();

	// Called to request that low level allocator must refreshed
	static CORE_API FSimpleMulticastDelegate& GetRefreshLowLevelAllocatorDelegate();

	// Called when OOM event occurs, after backup memory has been freed, so there's some hope of being effective
	static CORE_API FSimpleMulticastDelegate& GetOutOfMemoryDelegate();

	// Called from TerminateOnOutOfMemory in D3D11Util.cpp/D3D12Util.cpp
	static CORE_API TMulticastDelegate<void(const uint64, const uint64)>& GetGPUOutOfMemoryDelegate();

	enum class EOnScreenMessageSeverity : uint8
	{
		Info,
		Warning,
		Error,
	};
	typedef TMultiMap<EOnScreenMessageSeverity, FText> FSeverityMessageMap;

	// Called when displaying on screen messages (like the "Lighting needs to be rebuilt"), to let other systems add any messages as needed
	// Sample Usage:
	// void GetMyOnScreenMessages(FCoreDelegates::FSeverityMessageMap& OutMessages)
	// {
	//		OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::Format(LOCTEXT("MyMessage", "My Status: {0}"), SomeStatus));
	// }
	static CORE_API TMulticastDelegate<void(FSeverityMessageMap&)> OnGetOnScreenMessages;

	static CORE_API TDelegate<bool()> IsLoadingMovieCurrentlyPlaying;

	// Callback to allow user code to prevent url from being launched from FPlatformProcess::LaunchURL. Used to apply http allow list
	// Return true for to launch the url
	static CORE_API TDelegate<bool(const TCHAR* URL)> ShouldLaunchUrl;

	// Callback to allow user code to implement a custom implementation for FPlatformProcess::LaunchURL. (Windows Only)
	static CORE_API TDelegate<void(const FString& URL, FString* Error)> LaunchCustomHandlerForURL;
	
	// Callback when the application has been activated by protocol (with optional user id, depending on the platform)
	static CORE_API TMulticastDelegate<void(const FString& Parameter, FPlatformUserId UserId /*= PLATFORMUSERID_NONE*/)> OnActivatedByProtocol;

	/** Sent when GC finish destroy takes more time than expected */
	static CORE_API TMulticastDelegate<void(const FString&)> OnGCFinishDestroyTimeExtended;

	/** Called when the application's network initializes or shutdowns on platforms where the network stack is not always available */
	static CORE_API TMulticastDelegate<void(bool bIsNetworkInitialized)> ApplicationNetworkInitializationChanged;

	/**
	 * Called when the connection state as reported by the platform changes
	 *
	 * @param LastConnectionState last state of the connection
	 * @param ConnectionState current state of the connection
	 */
	static CORE_API TMulticastDelegate<void(ENetworkConnectionStatus LastConnectionState, ENetworkConnectionStatus ConnectionState)> OnNetworkConnectionStatusChanged;

	// Called when the application audio focus changed. Only called if there's audio focus event listeners
	static CORE_API TMulticastDelegate<void(bool bHasAudioFocus)> AudioFocusChangedDelegate;

	// Callback to let code read or write specialized binary data that is generated at Stage time, for optimizing data right before 
	// final game data is being written to disk
	// The TMap is a map of an identifier for owner of the data, and a boolean where true means the data is being generated (ie editor), and false 
	// means the data is for use (ie runtime game)
	struct FExtraBinaryConfigData
	{
		// the data that will be saved/loaded quickly
		TMap<FString, TArray<uint8>> Data;

		// Ini config data (not necessarily GConfig)
		class FConfigCacheIni& Config;

		// if true, the callback should fill out Data/Config
		bool bIsGenerating;

		FExtraBinaryConfigData(class FConfigCacheIni& InConfig, bool InIsGenerating)
			: Config(InConfig)
			, bIsGenerating(InIsGenerating)
		{
		}
	};
	

	/** Callback triggered when a shader map hash has been resolved to an I/O store chunk ID. */
	using FIoChunkIdResolvedFunc = TFunctionRef<void(const class FIoChunkId&)>;

	/**
	 * Callback to resolve I/O store chunk IDs from shader code hashes.
	 * Used when installing content on-demand.
	 *
	 * @param	Hashes		Array of hashes that should be resolved.
	 * @param	Callback	Callback triggered for each resolved chunk ID.
	 */
	static CORE_API TDelegate<void(TArrayView<const FShaderHash>, FIoChunkIdResolvedFunc)> ResolvePackageShaderMaps;

	static CORE_API TTSMulticastDelegate<void(FExtraBinaryConfigData&)>& TSAccessExtraBinaryConfigData();

#if USE_MMAPPED_SHADERARCHIVE
	using FShaderReadRequestResult = TPair<class FIoRequest, TIoStatusOr<struct FIoMappedRegion>>;
#else
	using FShaderReadRequestResult = class FIoRequest;
#endif
	using FAttachShaderReadRequestFunc = TFunctionRef<FShaderReadRequestResult(const class FIoChunkId&, FGraphEventRef)>;
	static CORE_API TDelegate<void(TArrayView<const FShaderHash>, FAttachShaderReadRequestFunc)> PreloadPackageShaderMaps;
	static CORE_API TDelegate<void(TArrayView<const FShaderHash>)> ReleasePreloadedPackageShaderMaps;
	/** Called when the verbosity of a log category is changed */
	static CORE_API TMulticastDelegate<void(const FLogCategoryName& CategoryName, ELogVerbosity::Type OldVerbosity, ELogVerbosity::Type NewVerbosity)> OnLogVerbosityChanged;

	// Called immediately before the parent process will start responding to signals to fork
	static CORE_API FSimpleMulticastDelegate OnParentBeginFork;
	// Called each time immediately before the parent process forks itself
	static CORE_API FSimpleMulticastDelegate OnParentPreFork;

	// Called immediately after the process spawned a fork
	static CORE_API TMulticastDelegate<void(EForkProcessRole ProcessRole)> OnPostFork;
	// Called at the end of the frame where the process spawned a fork
	static CORE_API FSimpleMulticastDelegate OnChildEndFramePostFork;

	// Called when FParse::Command succeeds. Used to audit named commands usage.
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNamedCommandParsed, const TCHAR* /*Cmd*/);
	static CORE_API FOnNamedCommandParsed OnNamedCommandParsed;

	// Called when FExec::Exec is called in a context where it's disallowed (UE_ALLOW_EXEC_COMMANDS = 0).
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDisallowedExecCommandCalled, const TCHAR* /*Cmd*/);
	static CORE_API FOnDisallowedExecCommandCalled OnDisallowedExecCommandCalled;

	// Extension point for projects to report the URL for a continuous integration job which produced these binaries.
	static CORE_API TDelegate<const TCHAR*()> OnGetBuildURL;
	
	// Extension point for projects to report the URL for a continuous integration job which is currently executing this process. 
	static CORE_API TDelegate<const TCHAR*()> OnGetExecutingJobURL;
	
#if WITH_EDITOR
	// Called when a subprocess is created for multiprocess operation.
	static CORE_API TMulticastDelegate<void(const UE::FMultiprocessCreatedContext&)> OnMultiprocessWorkerCreated;

	// Called when a subprocess is detached (but not necessarily terminated) for multiprocess operation.
	static CORE_API TMulticastDelegate<void(const UE::FMultiprocessDetachedContext&)> OnMultiprocessWorkerDetached;
#endif

	// Called during each FLowLevelMemTracker tick.
	static CORE_API FSimpleMulticastDelegate OnLowLevelMemTrackerTick;

private:
	// Callbacks for hotfixes
	static TArray<TDelegate<void(void*, int32)>> HotFixDelegates;

	// This class is only for namespace use
	FCoreDelegates() = default;
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Experimental/UnifiedError/UnifiedError.h"
#endif
