// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "HAL/PlatformProperties.h"
#include "HAL/ThreadSingleton.h"
#include "Logging/LogVerbosity.h"
#include "Math/IntPoint.h"
#include "Misc/Build.h"
#include "Misc/Exec.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"

class FOutputDevice;
class UWorld;

/**
 * Exec handler that registers itself and is being routed via StaticExec.
 * Note: Not intended for use with UObjects!
 */
class FSelfRegisteringExec : public FExec
{
public:
	/** Constructor, registering this instance. */
	CORE_API FSelfRegisteringExec();
	/** Destructor, unregistering this instance. */
	CORE_API virtual ~FSelfRegisteringExec();

	/** Routes a command to the self-registered execs. */
	static CORE_API bool StaticExec( UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar );
};

/** Registers a static Exec function using FSelfRegisteringExec. */
class FStaticSelfRegisteringExec : public FSelfRegisteringExec
{
public:

	/** Initialization constructor. */
	CORE_API FStaticSelfRegisteringExec(bool (*InStaticExecFunc)(UWorld* Inworld, const TCHAR* Cmd,FOutputDevice& Ar));

	//~ Begin Exec Interface
#if UE_ALLOW_EXEC_COMMANDS
	CORE_API virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) override;
#endif
	//~ End Exec Interface

private:

	bool (*StaticExecFunc)(UWorld* Inworld, const TCHAR* Cmd,FOutputDevice& Ar);
};

/** Registers a static Exec_Dev function using FSelfRegisteringExec. */
class FStaticSelfRegisteringExec_Dev: public FSelfRegisteringExec
{
public:

	/** Initialization constructor. */
	CORE_API explicit FStaticSelfRegisteringExec_Dev(bool (*InStaticExecFunc)(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar));

	//~ Begin Exec Interface
	CORE_API virtual bool Exec_Dev(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	//~ End Exec Interface

private:

	bool (*StaticExecFunc)(UWorld* Inworld, const TCHAR* Cmd,FOutputDevice& Ar);
};

/** Registers a static Exec_Editor function using FSelfRegisteringExec. */
class FStaticSelfRegisteringExec_Editor: public FSelfRegisteringExec
{
public:

	/** Initialization constructor. */
	CORE_API explicit FStaticSelfRegisteringExec_Editor(bool (*InStaticExecFunc)(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar));

	//~ Begin Exec Interface
	CORE_API virtual bool Exec_Editor(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	//~ End Exec Interface

private:

	bool (*StaticExecFunc)(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar);
};

// Interface for returning a context string.
class FContextSupplier
{
public:
	virtual ~FContextSupplier() = default;
	virtual FString GetContext()=0;
};


struct FMaintenance
{
	/** deletes log files older than a number of days specified in the Engine ini file */
	static CORE_API void DeleteOldLogs();
};

/*-----------------------------------------------------------------------------
	Module singletons.
-----------------------------------------------------------------------------*/

/** Returns the derived data cache interface if it is available, otherwise null. */
CORE_API class FDerivedDataCacheInterface* GetDerivedDataCache();

/** Returns the derived data cache interface, or fatal error if it is not available. */
CORE_API class FDerivedDataCacheInterface& GetDerivedDataCacheRef();

/** Returns the derived data cache interface if it is available and initialized, otherwise null. */
CORE_API class FDerivedDataCacheInterface* TryGetDerivedDataCache();

/**
 * Return the Target Platform Manager interface, if it is available, otherwise return nullptr.
 *
 * @param bFailOnInitErrors If true (default) and errors occur during init of the TPM, an error will be logged and the process may terminate, otherwise will return whether there was an error or not.
 * @return The Target Platform Manager interface, if it is available, otherwise return nullptr.
*/
CORE_API class ITargetPlatformManagerModule* GetTargetPlatformManager(bool bFailOnInitErrors = true);

/** Return the Target Platform Manager interface, fatal error if it is not available. **/
CORE_API class ITargetPlatformManagerModule& GetTargetPlatformManagerRef();

/**
 * Return true if we are currently in a commandlet is targeting platforms with AV requirements (ie not a server) 
 * or we are not targetingother platforms, and the current platform needs to render (CanEverRender())
 */
CORE_API bool WillNeedAudioVisualData();

/*-----------------------------------------------------------------------------
	Runtime.
-----------------------------------------------------------------------------*/

#if UE_EDITOR
CORE_API int32 StaticDedicatedServerCheck(); // Internal, use IsRunningDedicatedServer()
CORE_API int32 StaticGameCheck(); // Internal, use IsRunningGame()
CORE_API int32 StaticClientOnlyCheck(); // Internal, use IsRunningClientOnly()
#endif

/**
 * Check to see if this executable was launched as a dedicated server process and should not load client only data.
 * An editor build can be launched with -server to set this to true, but it will be false during single process PlayInEditor mode.
 * This function should not be used for gameplay or networking purposes, check for NM_DedicatedServer instead.
 */
inline bool IsRunningDedicatedServer()
{
	if (FPlatformProperties::IsServerOnly())
	{
		return true;
	}

	if (FPlatformProperties::IsGameOnly())
	{
		return false;
	}

#if UE_EDITOR
	return (StaticDedicatedServerCheck() == 1);
#else
	return false;
#endif
}

/**
 * Check to see if this executable was launched as a game (not editor or dedicated server) process. 
 * This is true for both client only and client/server packaged games, call IsRunningClientOnly to differentiate.
 * An editor build can be launched with -game to set this to true, but it will be false during single process PlayInEditor mode.
 * This function should not be used for gameplay or networking purposes, check the NetMode instead.
 */
inline bool IsRunningGame()
{
	if (FPlatformProperties::IsGameOnly())
	{
		return true;
	}

	if (FPlatformProperties::IsServerOnly())
	{
		return false;
	}

#if UE_EDITOR
	return (StaticGameCheck() == 1);
#else
	return false;
#endif
}

/**
 * Check to see if this executable was launched as a client only game process that should not load server data.
 * This will be true for packaged builds with a Client target type which will define WITH_SERVER_CODE=0.
 * An editor build can be launched with -clientonly to set this to true, but it will be false during single process PlayInEditor mode.
 * This function should not be used for gameplay or networking purposes, check for NM_Client instead.
 */
inline bool IsRunningClientOnly()
{
	if (FPlatformProperties::IsClientOnly())
	{
		return true;
	}

#if UE_EDITOR
	return (StaticClientOnlyCheck() == 1);
#else
	return false;
#endif
}

/**
 * Helper for obtaining the default Url configuration
 */
struct FUrlConfig
{
	FString DefaultProtocol;
	FString DefaultName;
	FString DefaultHost;
	FString DefaultPortal;
	FString DefaultSaveExt;
	int32 DefaultPort;

	/**
	 * Initialize with defaults from ini
	 */
	CORE_API void Init();

	/**
	 * Reset state
	 */
	CORE_API void Reset();
};

bool CORE_API StringHasBadDashes(const TCHAR* Str);

/** Helper structure for boolean values in config */
struct FBoolConfigValueHelper
{
private:
	bool bValue;
public:
	CORE_API FBoolConfigValueHelper(const TCHAR* Section, const TCHAR* Key, const FString& Filename = GEditorIni);

	operator bool() const
	{
		return bValue;
	}
};

/**
 * Function signature for handlers for script exceptions.
 */
typedef TFunction<void(ELogVerbosity::Type /*Verbosity*/, const TCHAR* /*ExceptionMessage*/, const TCHAR* /*StackMessage*/)> FScriptExceptionHandlerFunc;

UE_DECLARE_THREAD_SINGLETON_TLS(class FScriptExceptionHandler, CORE_API)

/** 
 * Exception handler stack used for script exceptions.
 */
class FScriptExceptionHandler : public TThreadSingleton<FScriptExceptionHandler>
{
public:
	/**
	 * Get the exception handler for the current thread
	 */
	static CORE_API FScriptExceptionHandler& Get();

	/**
	 * Push an exception handler onto the stack
	 */
	CORE_API void PushExceptionHandler(const FScriptExceptionHandlerFunc& InFunc);

	/**
	 * Pop an exception handler from the stack
	 */
	CORE_API void PopExceptionHandler();

	/**
	 * Handle an exception using the active exception handler
	 */
	CORE_API void HandleException(ELogVerbosity::Type Verbosity, const TCHAR* ExceptionMessage, const TCHAR* StackMessage);

	/**
	 * Handler for a script exception that emits an ensure (for warnings or errors)
	 */
	static CORE_API void AssertionExceptionHandler(ELogVerbosity::Type Verbosity, const TCHAR* ExceptionMessage, const TCHAR* StackMessage);

	/**
	 * Handler for a script exception that emits a log message
	 */
	static CORE_API void LoggingExceptionHandler(ELogVerbosity::Type Verbosity, const TCHAR* ExceptionMessage, const TCHAR* StackMessage);

private:
	/**
	 * Default script exception handler
	 */
	static FScriptExceptionHandlerFunc DefaultExceptionHandler;

	/**
	 * Stack of active exception handlers
	 * The top of the stack will be called on an exception, or DefaultExceptionHandler will be used if the stack is empty
	 */
	TArray<FScriptExceptionHandlerFunc, TInlineAllocator<4>> ExceptionHandlerStack;
};

/** 
 * Scoped struct used to push and pop a script exception handler
 */
struct FScopedScriptExceptionHandler
{
	CORE_API explicit FScopedScriptExceptionHandler(const FScriptExceptionHandlerFunc& InFunc);
	CORE_API ~FScopedScriptExceptionHandler();
};

/**
 * Enables named events when profiling.
 * Increments/decrements GCycleStatsShouldEmitNamedEvents based on bIsProfiling.
 * Functionality is controlled by stats.AutoEnableNamedEventsWhenProfiling.
 */
class FAutoNamedEventsToggler
{
public:
	CORE_API void Update(bool bIsProfiling);

private:
	bool bSetNamedEventsEnabled = false;
};

/** 
 * This define enables the blueprint runaway and exception stack trace checks
 * If this is true, it will create a FBlueprintContextTracker (previously FBlueprintExceptionTracker) which is defined in Script.h
 */
#ifndef DO_BLUEPRINT_GUARD
	#if (!(UE_BUILD_SHIPPING || UE_BUILD_TEST) || WITH_EDITOR)
		#define DO_BLUEPRINT_GUARD 1
	#else
		#define DO_BLUEPRINT_GUARD 0
	#endif
#endif

/** This define enables ScriptAudit exec commands */
#ifndef SCRIPT_AUDIT_ROUTINES
	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		#define SCRIPT_AUDIT_ROUTINES 1
	#else
		#define SCRIPT_AUDIT_ROUTINES 0
	#endif
#endif
