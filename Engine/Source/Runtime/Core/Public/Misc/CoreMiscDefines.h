// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HAL/PreprocessorHelpers.h"
#include "UObject/CoreReflectedTypeAccessors.h"

// When passed to pragma message will result in clickable warning in VS
#define WARNING_LOCATION(Line) __FILE__ "(" UE_STRINGIZE(Line) ")"

// This file is included in some resource files, which issue a warning:
//
// warning RC4011: identifier truncated to 'PLATFORM_CAN_SUPPORT_EDITORONLY'
//
// due to limitations of resource compiler.
#ifndef RC_INVOKED

#define PLAYWORLD_PACKAGE_PREFIX TEXT("UEDPIE")

#ifndef WITH_EDITORONLY_DATA
	#if !PLATFORM_CAN_SUPPORT_EDITORONLY_DATA || UE_SERVER || PLATFORM_IOS
		#define WITH_EDITORONLY_DATA	0
	#else
		#define WITH_EDITORONLY_DATA	1
	#endif
#endif

/** This controls if metadata for compiled in classes is unpacked and setup at boot time. Meta data is not normally used except by the editor. **/
#ifndef WITH_METADATA
#define WITH_METADATA WITH_EDITORONLY_DATA
#elif WITH_EDITORONLY_DATA && !WITH_METADATA
#error WITH_EDITORONLY_DATA=1 requires WITH_METADATA=1
#endif

// Option to check for UE_DISABLE_OPTIMIZATION being submitted
#ifndef UE_CHECK_DISABLE_OPTIMIZATION
#define UE_CHECK_DISABLE_OPTIMIZATION 0
#endif

// Set up optimization control macros, now that we have both the build settings and the platform macros

// Defines for submitting optimizations off
#define UE_DISABLE_OPTIMIZATION_SHIP  PRAGMA_DISABLE_OPTIMIZATION_ACTUAL

//in debug keep optimizations off for the enable macro otherwise code following enable will be optimized
#if UE_BUILD_DEBUG
	#define UE_ENABLE_OPTIMIZATION_SHIP  PRAGMA_DISABLE_OPTIMIZATION_ACTUAL
#else
	#define UE_ENABLE_OPTIMIZATION_SHIP  PRAGMA_ENABLE_OPTIMIZATION_ACTUAL
#endif

// if running on a build machine assert on the dev optimizations macros to validate that code is not being submitted with optimizations off
#if UE_CHECK_DISABLE_OPTIMIZATION
	#define UE_DISABLE_OPTIMIZATION static_assert(false, "Error UE_DISABLE_OPTIMIZATION submitted. Use UE_DISABLE_OPTIMIZATION_SHIP to submit with optimizations off.");
	#define UE_ENABLE_OPTIMIZATION static_assert(false, "Error UE_ENABLE_OPTIMIZATION submitted. Use UE_ENABLE_OPTIMIZATION_SHIP to submit with optimizations off.");
#else
	#define UE_DISABLE_OPTIMIZATION  UE_DISABLE_OPTIMIZATION_SHIP
	#define UE_ENABLE_OPTIMIZATION  UE_ENABLE_OPTIMIZATION_SHIP
#endif

#define PRAGMA_DISABLE_OPTIMIZATION \
	UE_DEPRECATED_MACRO(5.2, "PRAGMA_DISABLE_OPTIMIZATION has been deprecated. Use UE_DISABLE_OPTIMIZATION for temporary development or UE_DISABLE_OPTIMIZATION_SHIP to submit") \
	UE_DISABLE_OPTIMIZATION_SHIP

#define PRAGMA_ENABLE_OPTIMIZATION  \
	UE_DEPRECATED_MACRO(5.2, "PRAGMA_ENABLE_OPTIMIZATION has been deprecated. Use UE_ENABLE_OPTIMIZATION for temporary development or UE_ENABLE_OPTIMIZATION_SHIP to submit") \
	UE_ENABLE_OPTIMIZATION_SHIP

#if UE_BUILD_DEBUG
	#define FORCEINLINE_DEBUGGABLE FORCEINLINE_DEBUGGABLE_ACTUAL
#else
	#define FORCEINLINE_DEBUGGABLE FORCEINLINE
#endif

// Macro to disable inlining of function calls within the statement that follows
// e.g: UE_DONT_INLINE_CALL ThisFunctionCallWontBeInlined(Foo->ThisIsntInlinedEither());
#if !defined(UE_DONT_INLINE_CALL)
	#if defined(__clang__) && defined(__has_cpp_attribute) && __has_cpp_attribute(clang::noinline)
		#define UE_DONT_INLINE_CALL [[clang::noinline]]
	#else
		#define UE_DONT_INLINE_CALL
	#endif
#endif


#if STATS
	#define CLOCK_CYCLES(Timer)   {Timer -= FPlatformTime::Cycles();}
	#define UNCLOCK_CYCLES(Timer) {Timer += FPlatformTime::Cycles();}
#else
	#define CLOCK_CYCLES(Timer)
	#define UNCLOCK_CYCLES(Timer)
#endif

#define SHUTDOWN_IF_EXIT_REQUESTED
#define RETURN_IF_EXIT_REQUESTED
#define RETURN_VAL_IF_EXIT_REQUESTED(x)

#if CHECK_PUREVIRTUALS
#define PURE_VIRTUAL(func,...) =0;
#else
#define PURE_VIRTUAL(func,...) { LowLevelFatalError(TEXT("Pure virtual not implemented (%s)"), TEXT(#func)); __VA_ARGS__ }
#endif


// Code analysis features
#ifndef USING_CODE_ANALYSIS
	#define USING_CODE_ANALYSIS 0
#endif

#if USING_CODE_ANALYSIS
	#if !defined( CA_IN ) || !defined( CA_OUT ) || !defined( CA_READ_ONLY ) || !defined( CA_WRITE_ONLY ) || !defined( CA_VALID_POINTER ) || !defined( CA_CHECK_RETVAL ) || !defined( CA_NO_RETURN ) || !defined( CA_SUPPRESS ) || !defined( CA_ASSUME )
		#error Code analysis macros are not configured correctly for this platform
	#endif
#else
	// Just to be safe, define all of the code analysis macros to empty macros
	#define CA_IN 
	#define CA_OUT
	#define CA_READ_ONLY
	#define CA_WRITE_ONLY
	#define CA_VALID_POINTER
	#define CA_CHECK_RETVAL
	#define CA_NO_RETURN
	#define CA_SUPPRESS( WarningNumber )
	#define CA_ASSUME( Expr ) ((void)sizeof((bool)(Expr)))
	#define CA_CONSTANT_IF(Condition) if (Condition)
#endif

#ifndef USING_INSTRUMENTATION
	#define USING_INSTRUMENTATION 0
#endif

#if USING_THREAD_SANITISER || USING_INSTRUMENTATION
	#if !defined( TSAN_SAFE ) || !defined( TSAN_BEFORE ) || !defined( TSAN_AFTER ) || !defined( TSAN_ATOMIC )
		#error Thread Sanitiser macros are not configured correctly for this platform
	#endif
#else
	// Define TSAN macros to empty when not enabled
	#define TSAN_SAFE
	#define TSAN_BEFORE(Addr)
	#define TSAN_AFTER(Addr)
	#define TSAN_ATOMIC(Type) Type
#endif

enum {INDEX_NONE	= -1				};
enum {UNICODE_BOM   = 0xfeff			};

enum EForceInit 
{
	ForceInit,
	ForceInitToZero
};
enum ENoInit {NoInit};
// Used to add an explicitly consteval constructor when the default constructor for a type cannot
// be made constexpr (e.g. we are attempting to avoid zero-initialization when constructing at runtime)
enum EConstEval {ConstEval};
enum EInPlace {InPlace};
enum EPerElement {PerElement};

#endif // RC_INVOKED

// Push and pop macro definitions
#ifdef __clang__
	#define UE_PUSH_MACRO(name) _Pragma(UE_STRINGIZE(push_macro(name)))
	#define UE_POP_MACRO(name) _Pragma(UE_STRINGIZE(pop_macro(name)))
#else
	#define UE_PUSH_MACRO(name) __pragma(push_macro(name))
	#define UE_POP_MACRO(name) __pragma(pop_macro(name))
#endif

#ifdef __COUNTER__
	// Created a variable with a unique name
	#define ANONYMOUS_VARIABLE( Name ) UE_JOIN(Name, __COUNTER__)
#else
	// Created a variable with a unique name.
	// Less reliable than the __COUNTER__ version.
	#define ANONYMOUS_VARIABLE( Name ) UE_JOIN(Name, __LINE__)
#endif

/** Thread-safe call once helper for void functions, similar to std::call_once without the std::once_flag */
#define UE_CALL_ONCE(Func, ...) static int32 ANONYMOUS_VARIABLE(ThreadSafeOnce) = ((Func)(__VA_ARGS__), 1)

#if defined (__INTELLISENSE__)

	#define UE_DEPRECATED(Version, Message)
	#define UE_INTERNAL
	#define PRAGMA_DISABLE_INTERNAL_WARNINGS 
	#define PRAGMA_ENABLE_INTERNAL_WARNINGS 
	#define UE_EXPERIMENTAL(Version, Message)
	#define PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS 
	#define PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS 

#else // defined (__INTELLISENSE__)
	
/**
 * Macro for marking up deprecated code, functions and types.
 *
 * This should be used as syntactic replacement for the [[deprecated]] attribute
 * which provides a UE version number like the old DEPRECATED macro.
 *
 * Features that are marked as deprecated are scheduled to be removed from the code base
 * in a future release. If you are using a deprecated feature in your code, you should
 * replace it before upgrading to the next release. See the Upgrade Notes in the release
 * notes for the release in which the feature was marked deprecated.
 *
 * Sample usage (note the slightly different syntax for classes and structures):
 *
 *		UE_DEPRECATED(5.xx, "MyFunction has been deprecated, please use MyNewFunction instead.")
 *		void MyFunction();
 *
 *		UE_DEPRECATED(5.xx, "FMyType has been deprecated, please use MyNewType instead.")
 *		typedef FThing FMyType;
 *
 *		using FMyAlias UE_DEPRECATED(5.xx, "FMyAlias has been deprecated, please use MyNewAlias instead.") = FThing;
 *
 *		UE_DEPRECATED(5.xx, "MyVariable has been deprecated and serves no purpose anymore")
 *		int32 MyVariable;
 *
 *		namespace UE_DEPRECATED(5.xx, "MyNamespace has been renamed MyNewNamespace, please use this instead.") MyNamespace
 *		{
 *		}
 *
 *		Unfortunately, clang will complain that [the] "declaration of [an] anonymous class must
 *		be a definition" for API types.  To work around this, first forward declare the type as
 *		deprecated, then declare the type with the visibility macro.  Note that macros like
 *		USTRUCT must immediately precede the the type declaration, not the forward declaration.
 *
 *		struct UE_DEPRECATED(5.xx, "FMyStruct has been deprecated and shall not be used anymore.") FMyStruct;
 *		USTRUCT()
 *		struct MODULE_API FMyStruct
 *		{
 *		};
 *
 *		class UE_DEPRECATED(5.xx, "FMyClass has been deprecated and shall not be used anymore.") FMyClass;
 *		class MODULE_API FMyClass
 *		{
 *		};
 *
 *		enum class UE_DEPRECATED(5.xx, "EMyEnumeration has been deprecated, use EMyEnumeration instead.") EMyEnumeration
 *		{
 *			Zero = 0,
 *			One UE_DEPRECATED(5.xx, "EMyEnumeration::One is not a valid enum entry anymore, please use another one.") = 1,
 *			Two = 2
 *		};
 *
 *		Unfortunately, VC++ will complain about using member functions and fields from deprecated
 *		class/structs even for class/struct implementation e.g.:
 *		class UE_DEPRECATED(5.xx, "DeprecatedClass has been deprecated and shall not be used anymore.") DeprecatedClass
 *		{
 *		public:
 *			DeprecatedClass() {}
 *
 *			float GetMyFloat()
 *			{
 *				return MyFloat; // This line will cause warning that deprecated field is used.
 *			}
 *		private:
 *			float MyFloat;
 *		};
 *
 *		To get rid of this warning, place all code not called in class implementation in non-deprecated
 *		base class and deprecate only derived one. This may force you to change some access specifiers
 *		from private to protected, e.g.:
 *
 *		class DeprecatedClass_Base_DEPRECATED
 *		{
 *		protected: // MyFloat is protected now, so DeprecatedClass has access to it.
 *			float MyFloat;
 *		};
 *
 *		class UE_DEPRECATED(5.xx, "DeprecatedClass has been deprecated and shall not be used anymore.") DeprecatedClass : DeprecatedClass_Base_DEPRECATED
 *		{
 *		public:
 *			DeprecatedClass() {}
 *
 *			float GetMyFloat()
 *			{
 *				return MyFloat;
 *			}
 *		};
 *
 *		template <typename T>
 *		class UE_DEPRECATED(5.xx, "DeprecatedClassTemplate has been deprecated and shall not be used anymore.") DeprecatedClassTemplate
 *		{
 *		};
 *
 *		template <typename T>
 *		UE_DEPRECATED(5.xx, "DeprecatedFunctionTemplate has been deprecated and shall not be used anymore.")
 *		void DeprecatedFunctionTemplate()
 *		{
 *		}
 *
 * @param Version The release number in which the feature was marked deprecated. Note: 'all', as in UE_DEPRECATED(all, ...), can be used for reflected code which has to stick around forever and 
    is marked with meta-data like DeprecatedFunction or DeprecatedProperty. It indicates that the deprecation grace period is now over but that this property/function will remain deprecated for all time.
 * @param Message A message containing upgrade notes.
 */
	#define UE_DEPRECATED(Version, Message) [[deprecated(Message " - Please update your code to the new API before upgrading to the next release, otherwise your project will no longer compile.")]]

/**
 * An alternative use of the deprecation feature is to note an API breaking change that could cause data loss or similar, so we use this when 
 * tagging a property that is a compile error to use (for instance by changing its type to FDeprecationCompileError type, which is incompatible with 
 * anything), but it can still show the deprecation message telling users how to fix their code. This should only be used when a nice, clean 
 * deprecation is not possible.
 * YOU MUST FOLLOW THIS WITH SOMETHING THAT CAUSES AN ERROR - for instance, change the variable type to FDeprecationCompileError so any use will fail,
 * or use one of the below helpers
 */
	struct FDeprecationCompileError {};
	#define UE_DEPRECATED_ERROR(Version, Message) [[deprecated(Message " - ** This is a compile error so as to avoid data loss, or similar. ** You will need to update your code to get it to compile correctly.")]]
	
/** A version of the above that declares a compilation-error type with a message indicating it has moved and you 
 * need to use accessor functions instead.
 */
	#define UE_DEPRECATED_ERROR_WITH_MOVED_MEMBER_MESSAGE(Version, Member, Message) \
		[[deprecated(Message " ** This is a compile error so as to avoid data loss, or similar. ** You will need to update your code to get it to compile correctly.")]] \
		FDeprecationCompileError Member;
	
/** A version of the above that prints out a premade message with member name */
	#define UE_DEPRECATED_ERROR_WITH_MOVED_MEMBER(Version, Member) UE_DEPRECATED_ERROR_WITH_MOVED_MEMBER_MESSAGE(Version, Member, "The member " #Member " has moved. Use the Get/Set" #Member " accessors instead.")

/** This version prints out a message for the accessor without the b(SetFoo, not SetbFoo), but as such you need to pass in the bool Member without the b */
	#define UE_DEPRECATED_ERROR_WITH_MOVED_BOOL_MEMBER(Version, Member) UE_DEPRECATED_ERROR_WITH_MOVED_MEMBER_MESSAGE(Version, b##Member, "The member b" #Member " has moved. Use the Get/Set" #Member " accessors instead.")

 /**
 * Macro for marking up engine-internal code, functions and types. This informs the users that there is no guarantee that the API won't change or be removed in 
 *  future releases and that they should use at their risk. 
 *  Using an API marked as such in a non-engine module will trigger a compilation warning (this relies on the c++ [[deprecated]] attribute internally) that can be suppressed by either : 
 *   - Using PRAGMA_DISABLE_INTERNAL_WARNINGS and PRAGMA_ENABLE_INTERNAL_WARNINGS around the call site (preferred method)
 *   - Setting bValidateInternalApi to false in the calling module's build rules, thus telling the entire module that any internal API call is accepted
 *   - Setting bTreatAsEngineModule to true in the calling module's build rules, thus telling the entire module that any internal (OR experimental) API call is accepted
 * 
 * It is advised to simply avoid using APIs marked as such in non-engine modules. 
 *
 * Please refer to UE_DEPRECATED for how to use the macro. 
 */
	#if UE_VALIDATE_INTERNAL_API 
		#define UE_INTERNAL [[deprecated("This API is internal to the engine. As such, there is no guarantee that it won't change or be removed in future releases. Please use at your own risk.")]]
		#define PRAGMA_DISABLE_INTERNAL_WARNINGS PRAGMA_DISABLE_DEPRECATION_WARNINGS
		#define PRAGMA_ENABLE_INTERNAL_WARNINGS PRAGMA_ENABLE_DEPRECATION_WARNINGS
	#else // UE_VALIDATE_INTERNAL_API 
		#define UE_INTERNAL
		#define PRAGMA_DISABLE_INTERNAL_WARNINGS 
		#define PRAGMA_ENABLE_INTERNAL_WARNINGS 
	#endif // !UE_VALIDATE_INTERNAL_API 

/**
 * Macro for marking up experimental code, functions and types from non-experimental modules. This informs the users that there is no guarantee that the API won't change or be removed in 
 *  future releases and that they should use at their own risk. 
 *  Using an API marked as such in a non-engine module will trigger a compilation warning (this relies on the c++ [[deprecated]] attribute internally) that can be suppressed by either : 
 *   - Using PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS and PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS around the call site (preferred method)
 *   - Setting bValidateExperimentalApi to false in the calling module's build rules, thus telling the entire module that any experimental API call is accepted
 *   - Setting bTreatAsEngineModule to true in the calling module's build rules, thus telling the entire module that any experimental (OR internal) API call is accepted
 *
 * Please refer to UE_DEPRECATED for how to use the macro. 
 *
 * @param Version The release number in which the feature was introduced. This does *not* give an indication for how long this API will exist as "experimental". It is only there to inform 
 *  the users about when the experimental API was first introduced. After some time, this API may be removed or it may become official, at which point it will stop being marked as UE_EXPERIMENTAL.
 * @param Message A message that may contain additional details about the usage of this experimental API.
 */
	#if UE_VALIDATE_EXPERIMENTAL_API
		#define UE_EXPERIMENTAL(Version, Message)  [[deprecated(Message " - This API is experimental. As such, there is no guarantee that it won't change or be removed in future releases. Please use at your own risk.")]]
		#define PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS PRAGMA_DISABLE_DEPRECATION_WARNINGS
		#define PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS PRAGMA_ENABLE_DEPRECATION_WARNINGS
	#else // UE_VALIDATE_EXPERIMENTAL_API
		#define UE_EXPERIMENTAL(Version, Message)
		#define PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS 
		#define PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS 
	#endif // !UE_VALIDATE_EXPERIMENTAL_API

#endif // !defined(__INTELLISENSE__)

#ifndef UE_DEPRECATED_FORGAME
	#define UE_DEPRECATED_FORGAME UE_EMPTY_FUNCTION
#endif

#ifndef UE_DEPRECATED_FORENGINE
	#define UE_DEPRECATED_FORENGINE UE_DEPRECATED
#endif


/**
 * Macro which can be placed in a header to throw a deprecation warning when it is included.
 */
#if defined(UE_DIRECT_HEADER_COMPILE)
	// Don't warn about the deprecated header when we are directly compiling the header
	#define UE_DEPRECATED_HEADER(Version, Message)
#elif defined(_MSC_VER)
    #if UE_WARNINGS_AS_ERRORS
    	#define UE_DEPRECATED_HEADER(Version, Message) __pragma(message(__FILE__ "(" UE_STRINGIZE(__LINE__) "): error C4996: " Message " Please update your code to the new API before upgrading to the next release, otherwise your project will no longer compile."))
    #else
    	#define UE_DEPRECATED_HEADER(Version, Message) __pragma(message(__FILE__ "(" UE_STRINGIZE(__LINE__) "): warning C4996: " Message " Please update your code to the new API before upgrading to the next release, otherwise your project will no longer compile."))
    #endif
#else
	#define UE_DEPRECATED_HEADER(Version, Message) _Pragma(UE_STRINGIZE(message(Message " Please update your code to the new API before upgrading to the next release, otherwise your project will no longer compile.")))
#endif


/*
 * Macro that can be defined in the target file to strip deprecated properties in objects across the engine that check against this define.
 * Can be used by project that have migrated away from using deprecated functions and data members to potentially gain back some memory and perf.
 * @note This is a define that engine developer may use when deprecating properties to allow additional memory savings when a project is compliant with deprecation notice.
 * It doesn't indicate that all deprecated properties will be stripped.
 */
#ifndef UE_STRIP_DEPRECATED_PROPERTIES
	#define UE_STRIP_DEPRECATED_PROPERTIES 0
#endif

template <bool bIsDeprecated>
struct TStaticDeprecateExpression
{
};

/**
 * Can be used in the same contexts as static_assert but gives a warning rather than an error, and 'fails' if the expression is true rather than false.
 */
#define UE_STATIC_DEPRECATE(Version, bExpression, Message) \
	struct UE_JOIN(FDeprecationMsg_, __LINE__) { \
		[[deprecated(Message " - Please update your code to the new API before upgrading to the next release, otherwise your project will no longer compile.")]] \
		static constexpr int condition(TStaticDeprecateExpression<true>) { return 1; } \
		static constexpr int condition(TStaticDeprecateExpression<false>) { return 1; } \
	}; \
	enum class UE_JOIN(EDeprecationMsg_, __LINE__) { Value = UE_JOIN(FDeprecationMsg_, __LINE__)::condition(TStaticDeprecateExpression<!!(bExpression)>()) }

/**
 * Can be used in the same contexts as static_assert but gives a warning rather than an error
 */
#define UE_STATIC_ASSERT_WARN(bExpression, Message) \
	struct UE_JOIN(FStaticWarningMsg_, __LINE__) { \
		[[deprecated(Message)]] \
		static constexpr int condition(TStaticDeprecateExpression<true>) { return 1; } \
		static constexpr int condition(TStaticDeprecateExpression<false>) { return 1; } \
	}; \
	enum class UE_JOIN(EStaticWarningMsg_, __LINE__) { Value = UE_JOIN(FStaticWarningMsg_, __LINE__)::condition(TStaticDeprecateExpression<!(bExpression)>()) }

// These defines are used to mark a difference between two pointers as expected to fit into the specified range
// while still leaving something searchable if the surrounding code is updated to work with a 64 bit count/range
// in the future
#define UE_PTRDIFF_TO_INT32(argument) static_cast<int32>(argument)
#define UE_PTRDIFF_TO_UINT32(argument) static_cast<uint32>(argument)

/**
* Makes a type non-copyable and non-movable by deleting copy/move constructors and assignment/move operators.
* The macro should be placed in the public section of the type for better compiler diagnostic messages.
* Example usage:
*
*	class FMyClassName
*	{
*	public:
*		UE_NONCOPYABLE(FMyClassName)
*		FMyClassName() = default;
*	};
*/
#define UE_NONCOPYABLE(TypeName) \
	TypeName(TypeName&&) = delete; \
	TypeName(const TypeName&) = delete; \
	TypeName& operator=(const TypeName&) = delete; \
	TypeName& operator=(TypeName&&) = delete;


/** 
 * Handle that defines a local user on this platform.
 * This used to be just a typedef int32 that was used interchangeably as ControllerId and LocalUserIndex.
 * Moving forward these will be allocated by the platform application layer.
 */
struct FPlatformUserId
{
	/** Sees if this is a valid user */
	UE_FORCEINLINE_HINT bool IsValid() const
	{
		return InternalId != INDEX_NONE;
	}

	/** Returns the internal id for debugging/etc */
	UE_FORCEINLINE_HINT int32 GetInternalId() const
	{
		return InternalId;
	}

	/** Explicit function to create from an internal id */
	inline static FPlatformUserId CreateFromInternalId(int32 InInternalId)
	{
		FPlatformUserId IdToReturn;
		IdToReturn.InternalId = InInternalId;
		return IdToReturn;
	}

	UE_FORCEINLINE_HINT bool operator==(const FPlatformUserId& Other) const
	{
		return InternalId == Other.InternalId;
	}

	UE_FORCEINLINE_HINT bool operator!=(const FPlatformUserId& Other) const
	{
		return InternalId != Other.InternalId;
	}

	UE_FORCEINLINE_HINT friend uint32 GetTypeHash(const FPlatformUserId& UserId)
	{
		return UserId.InternalId;
	}

	// This should be deprecated when the online code uniformly handles FPlatformUserId */
	// UE_DEPRECATED(5.x, "Implicit conversion to user index is deprecated, use FPlatformMisc::GetUserIndexForPlatformUser")
	UE_FORCEINLINE_HINT constexpr operator int32() const { return InternalId; }

private:
	/** Raw id, will be allocated by application layer */
	int32 InternalId = INDEX_NONE;
};

/** Static invalid platform user */
inline constexpr FPlatformUserId PLATFORMUSERID_NONE;

/**
 * Represents a single input device such as a gamepad, keyboard, or mouse.
 *
 * Has a globally unique identifier that is assigned by the IPlatformInputDeviceMapper
 */
struct FInputDeviceId
{
	/** Explicit function to create from an internal id */
	inline static FInputDeviceId CreateFromInternalId(int32 InInternalId)
	{
		FInputDeviceId IdToReturn;
		IdToReturn.InternalId = InInternalId;
		return IdToReturn;
	}
	
	UE_FORCEINLINE_HINT int32 GetId() const
	{
		return InternalId;
	}

	/** Sees if this is a valid input device */
	UE_FORCEINLINE_HINT bool IsValid() const
	{
		return InternalId >= 0;
	}

	UE_FORCEINLINE_HINT bool operator==(const FInputDeviceId& Other) const
	{
		return InternalId == Other.InternalId;
	}

	UE_FORCEINLINE_HINT bool operator!=(const FInputDeviceId& Other) const
	{
		return InternalId != Other.InternalId;
	}
	
	UE_FORCEINLINE_HINT bool operator<(const FInputDeviceId& Other) const
	{
		return InternalId < Other.InternalId;
	}

	UE_FORCEINLINE_HINT bool operator<=(const FInputDeviceId& Other) const
	{
		return InternalId <= Other.InternalId;
	}

	UE_FORCEINLINE_HINT bool operator>(const FInputDeviceId& Other) const
	{
		return InternalId > Other.InternalId;
	}

	UE_FORCEINLINE_HINT bool operator>=(const FInputDeviceId& Other) const
	{
		return InternalId >= Other.InternalId;
	}

	UE_FORCEINLINE_HINT friend uint32 GetTypeHash(const FInputDeviceId& InputId)
	{
		return InputId.InternalId;
	}
	
private:
	
	/**
	 * Raw id, will be allocated by application layer
	 * 
	 * @see IPlatformInputDeviceMapper::AllocateNewInputDeviceId
	 */
	int32 InternalId = INDEX_NONE;
};

/** Static invalid input device. */
inline constexpr FInputDeviceId INPUTDEVICEID_NONE;

/** Represents the connection status of a given FInputDeviceId */
enum class EInputDeviceConnectionState : uint8
{
	/** This is not a valid input device */
	Invalid,

	/** It is not known if this device is connected or not */
	Unknown,

	/** Device is definitely connected */
	Disconnected,

	/** Definitely connected and powered on */
	Connected
};

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<EInputDeviceConnectionState>();
#endif

/** Data about an input device's current state */
struct FPlatformInputDeviceState
{
	/** The platform user that this input device belongs to */
	FPlatformUserId OwningPlatformUser = PLATFORMUSERID_NONE;

	/** The connection state of this input device */
	EInputDeviceConnectionState ConnectionState = EInputDeviceConnectionState::Invalid;
};

/** Remote objects support */
#ifndef UE_WITH_REMOTE_OBJECT_HANDLE
	#define UE_WITH_REMOTE_OBJECT_HANDLE 0
#endif