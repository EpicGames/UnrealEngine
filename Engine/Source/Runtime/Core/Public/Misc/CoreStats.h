// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Stats/Stats.h"




/** Memory stats */
DECLARE_MEMORY_STAT_EXTERN(TEXT("Audio Memory Used"),STAT_AudioMemory,STATGROUP_Memory, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Precomputed Visibility Memory"),STAT_PrecomputedVisibilityMemory,STATGROUP_Memory, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("SkeletalMesh Vertex Memory"),STAT_SkeletalMeshVertexMemory,STATGROUP_Memory, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("SkeletalMesh Index Memory"),STAT_SkeletalMeshIndexMemory,STATGROUP_Memory, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("SkeletalMesh M.BlurSkinning Memory"),STAT_SkeletalMeshMotionBlurSkinningMemory,STATGROUP_Memory, CORE_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("VertexShader Memory"),STAT_VertexShaderMemory,STATGROUP_Memory, FPlatformMemory::MCR_Physical, CORE_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("PixelShader Memory"),STAT_PixelShaderMemory,STATGROUP_Memory, FPlatformMemory::MCR_Physical, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Navigation Memory"),STAT_NavigationMemory,STATGROUP_Memory, CORE_API);
/** PhysX memory tracking needs PHYSX_MEMORY_STATS enabled */
DECLARE_MEMORY_STAT_EXTERN(TEXT("PhysX Memory Used"),STAT_MemoryPhysXTotalAllocationSize,STATGROUP_Memory, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("ICU Memory Used"),STAT_MemoryICUTotalAllocationSize,STATGROUP_Memory, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("ICU Data File Memory Used"),STAT_MemoryICUDataFileAllocationSize,STATGROUP_Memory, CORE_API);

DECLARE_MEMORY_STAT_EXTERN(TEXT("Texture Memory Used"),STAT_TextureMemory,STATGROUP_Memory, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Reflection Capture Texture Memory"),STAT_ReflectionCaptureTextureMemory,STATGROUP_Memory, CORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Reflection Capture Memory"),STAT_ReflectionCaptureMemory,STATGROUP_Memory, CORE_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Total Render thread idle time"),STAT_RenderingIdleTime,STATGROUP_Threading, CORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Wait for GPU Query"),STAT_RenderingIdleTime_WaitingForGPUQuery,STATGROUP_Threading, CORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Wait for GPU Present"),STAT_RenderingIdleTime_WaitingForGPUPresent,STATGROUP_Threading, CORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Other Render Thread Sleep Time"),STAT_RenderingIdleTime_RenderThreadSleepTime,STATGROUP_Threading, CORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Rendering thread busy time"),STAT_RenderingBusyTime,STATGROUP_Threading, CORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Game thread idle time"),STAT_GameIdleTime,STATGROUP_Threading, CORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Game thread tick wait time"),STAT_GameTickWaitTime,STATGROUP_Threading, CORE_API);
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Game thread requested wait time"),STAT_GameTickWantedWaitTime,STATGROUP_Threading, CORE_API);
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Game thread additional wait time"),STAT_GameTickAdditionalWaitTime,STATGROUP_Threading, CORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Game TaskGraph Tasks"),STAT_TaskGraph_GameTasks,STATGROUP_Threading, CORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Game TaskGraph Stalls"),STAT_TaskGraph_GameStalls,STATGROUP_Threading, CORE_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Render Local Queue Stalls"),STAT_TaskGraph_RenderStalls,STATGROUP_Threading, CORE_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Other TaskGraph Tasks"),STAT_TaskGraph_OtherTasks,STATGROUP_Threading, CORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Other TaskGraph Stalls"),STAT_TaskGraph_OtherStalls,STATGROUP_Threading, CORE_API);

DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Percentage CPU utilization"),STAT_CPUTimePct,STATGROUP_Threading, CORE_API);
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Percentage CPU utilization (relative to one core)"),STAT_CPUTimePctRelative,STATGROUP_Threading, CORE_API);


/*-----------------------------------------------------------------------------
	CPU Stalls
-----------------------------------------------------------------------------*/

DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("CPU Stall - Sleep"), STAT_Sleep, STATGROUP_CPUStalls, EStatFlags::Verbose, CORE_API);
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("CPU Stall - Wait For Event" ), STAT_EventWait, STATGROUP_CPUStalls, EStatFlags::Verbose,);

/** The id will be stored as uint64 in the stat message. */
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN( TEXT( "CPU Stall - Wait For Event with ID" ), STAT_EventWaitWithId, STATGROUP_CPUStalls, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN( TEXT( "CPU Stall - Trigger For Event with ID" ), STAT_EventTriggerWithId, STATGROUP_CPUStalls, EStatFlags::Verbose, );
