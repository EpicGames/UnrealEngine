// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformString.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Char.h"
#include "Templates/TupleFwd.h"
#include "Templates/UnrealTypeTraits.h"
#include "Traits/IsCharType.h"

/** 
 * CRC hash generation for different types of input data
 **/
struct FCrc
{
	/** Lookup table with precalculated CRC values - slicing by 8 implementation */
	static CORE_API uint32 CRCTablesSB8[8][256];

	/** Initializes the CRC lookup table. Must be called before any of the
		CRC functions are used. */
	static CORE_API void Init();

	/** Generates CRC hash of the memory area. Prefer Hash/xxhash.h for new code, CRCs are not very good and fairly slow. */
	typedef uint32 (*MemCrc32Functor)( const void* Data, int32 Length, uint32 CRC );
	static CORE_API MemCrc32Functor MemCrc32Func;
	[[nodiscard]] static UE_FORCEINLINE_HINT uint32 MemCrc32(const void* Data, int32 Length, uint32 CRC = 0)
	{
		return MemCrc32Func(Data, Length, CRC);
	}

	/** Generates CRC hash of the element. Prefer Hash/xxhash.h for new code, CRCs are not very good and fairly slow. */
	template <typename T>
	[[nodiscard]] static uint32 TypeCrc32( const T& Data, uint32 CRC=0 )
	{
		return MemCrc32(&Data, sizeof(T), CRC);
	}

	/** String CRC. Prefer Hash/Fnv.h for new code, CRCs are not very good and fairly slow. */
	template <typename CharType>
	[[nodiscard]] static uint32 StrCrc32(const CharType* Data, uint32 CRC = 0)
	{
		// We ensure that we never try to do a StrCrc32 with a CharType of more than 4 bytes.  This is because
		// we always want to treat every CRC as if it was based on 4 byte chars, even if it's less, because we
		// want consistency between equivalent strings with different character types.
		static_assert(sizeof(CharType) <= 4, "StrCrc32 only works with CharType up to 32 bits.");

		CRC = ~CRC;
		while (CharType Ch = *Data++)
		{
			uint32 V = CRC ^ Ch;
			CRC =
				CRCTablesSB8[3][ V        & 0xFF] ^
				CRCTablesSB8[2][(V >>  8) & 0xFF] ^
				CRCTablesSB8[1][(V >> 16) & 0xFF] ^
				CRCTablesSB8[0][ V >> 24        ];
		}
		return ~CRC;
	}

	/** String CRC. Prefer Hash/Fnv.h for new code, CRCs are not very good and fairly slow. */
	template <typename CharType>
	[[nodiscard]] static uint32 StrCrc32Len(const CharType* Data, int32 Length, uint32 CRC = 0)
	{
		// We ensure that we never try to do a StrCrc32 with a CharType of more than 4 bytes.  This is because
		// we always want to treat every CRC as if it was based on 4 byte chars, even if it's less, because we
		// want consistency between equivalent strings with different character types.
		static_assert(sizeof(CharType) <= 4, "StrCrc32Len only works with CharType up to 32 bits.");

		CRC = ~CRC;
		for (int32 Idx = 0; Idx < Length; ++Idx)
		{
			uint32 V = CRC ^ Data[Idx];
			CRC =
				CRCTablesSB8[3][ V        & 0xFF] ^
				CRCTablesSB8[2][(V >>  8) & 0xFF] ^
				CRCTablesSB8[1][(V >> 16) & 0xFF] ^
				CRCTablesSB8[0][ V >> 24        ];
		}
		return ~CRC;
	}

	/**
	 * DEPRECATED
	 * These tables and functions are deprecated because they're using tables and implementations
	 * which give values different from what a user of a typical CRC32 algorithm might expect.
	 */

	/** lookup table with precalculated CRC values */
	static CORE_API uint32 CRCTable_DEPRECATED[256];
	/** lookup table with precalculated CRC values - slicing by 8 implementation */
	static CORE_API uint32 CRCTablesSB8_DEPRECATED[8][256];

	/** String CRC. */
	template <typename CharType>
	static inline uint32 StrCrc_DEPRECATED(const CharType* Data)
	{
		// make sure table is initialized
		check(CRCTable_DEPRECATED[1] != 0);

		uint32 CRC = 0xFFFFFFFF;
		while (*Data)
		{
			CharType C = *Data++;
			int32 CL = (C&255);
			CRC = (CRC << 8) ^ CRCTable_DEPRECATED[(CRC >> 24) ^ CL];
			int32 CH = (C>>8)&255;
			CRC = (CRC << 8) ^ CRCTable_DEPRECATED[(CRC >> 24) ^ CH];
		}
		return ~CRC;
	}

	/** String CRC. */
	template <typename CharType>
	static inline uint32 StrCrc_DEPRECATED(const int32 DataLen, const CharType* Data)
	{
		// make sure table is initialized
		check(CRCTable_DEPRECATED[1] != 0);

		uint32 CRC = 0xFFFFFFFF;
		for (int32 i = 0; i < DataLen; i++)
		{
			CharType C = *Data++;
			int32 CL = (C&255);
			CRC = (CRC << 8) ^ CRCTable_DEPRECATED[(CRC >> 24) ^ CL];
			int32 CH = (C>>8)&255;
			CRC = (CRC << 8) ^ CRCTable_DEPRECATED[(CRC >> 24) ^ CH];
		}
		return ~CRC;
	}

	/** Case insensitive string hash function. */
	template <typename CharType> static inline uint32 Strihash_DEPRECATED( const CharType* Data );
	template <typename CharType> static inline uint32 Strihash_DEPRECATED( const int32 DataLen, const CharType* Data );

	/** generates CRC hash of the memory area */
	static CORE_API uint32 MemCrc_DEPRECATED( const void* Data, int32 Length, uint32 CRC=0 );
};

template <>
inline uint32 FCrc::Strihash_DEPRECATED(const ANSICHAR* Data)
{
	// make sure table is initialized
	check(CRCTable_DEPRECATED[1] != 0);

	uint32 Hash=0;
	while( *Data )
	{
		ANSICHAR Ch = TChar<ANSICHAR>::ToUpper(*Data++);
		uint8 B  = Ch;
		Hash = ((Hash >> 8) & 0x00FFFFFF) ^ CRCTable_DEPRECATED[(Hash ^ B) & 0x000000FF];
	}
	return Hash;
}

template <>
inline uint32 FCrc::Strihash_DEPRECATED(const int32 DataLen, const ANSICHAR* Data)
{
	// make sure table is initialized
	check(CRCTable_DEPRECATED[1] != 0);

	uint32 Hash=0;
	for (int32 Idx = 0; Idx < DataLen; ++Idx)
	{
		ANSICHAR Ch = TChar<ANSICHAR>::ToUpper(*Data++);
		uint8 B  = Ch;
		Hash = ((Hash >> 8) & 0x00FFFFFF) ^ CRCTable_DEPRECATED[(Hash ^ B) & 0x000000FF];
	}
	return Hash;
}

template <>
inline uint32 FCrc::Strihash_DEPRECATED(const WIDECHAR* Data)
{
	// make sure table is initialized
	check(CRCTable_DEPRECATED[1] != 0);

	uint32 Hash=0;
	while( *Data )
	{
		WIDECHAR Ch = TChar<WIDECHAR>::ToUpper(*Data++);
		uint16  B  = (uint16)Ch;
		Hash     = ((Hash >> 8) & 0x00FFFFFF) ^ CRCTable_DEPRECATED[(Hash ^ B) & 0x000000FF];
		B        = (uint16)Ch>>8;
		Hash     = ((Hash >> 8) & 0x00FFFFFF) ^ CRCTable_DEPRECATED[(Hash ^ B) & 0x000000FF];
	}
	return Hash;
}

template <>
inline uint32 FCrc::Strihash_DEPRECATED(const int32 DataLen, const WIDECHAR* Data)
{
	// make sure table is initialized
	check(CRCTable_DEPRECATED[1] != 0);

	uint32 Hash=0;
	for (int32 Idx = 0; Idx < DataLen; ++Idx)
	{
		WIDECHAR Ch = TChar<WIDECHAR>::ToUpper(*Data++);
		uint16  B  = (uint16)Ch;
		Hash     = ((Hash >> 8) & 0x00FFFFFF) ^ CRCTable_DEPRECATED[(Hash ^ B) & 0x000000FF];
		B        = (uint16)Ch>>8;
		Hash     = ((Hash >> 8) & 0x00FFFFFF) ^ CRCTable_DEPRECATED[(Hash ^ B) & 0x000000FF];
	}
	return Hash;
}

template <>
inline uint32 FCrc::Strihash_DEPRECATED(const UTF8CHAR* Data)
{
	// We can't utilize StringConv.h here due to circular includes, so do the conversion manually

	int32 Len          = FPlatformString::Strlen(Data);
	int32 ConvertedLen = FPlatformString::ConvertedLength<WIDECHAR>(Data, Len);

	WIDECHAR* Temp = new WIDECHAR[ConvertedLen];

	WIDECHAR* TempEnd = FPlatformString::Convert(Temp, ConvertedLen, Data, Len);
	checkf(TempEnd!=nullptr, TEXT("String conversion unsuccessful"));

	// This doesn't work for strings containing characters outside the BMP, but
	// then neither does the WIDECHAR overload.
	uint32 Result = FCrc::Strihash_DEPRECATED(ConvertedLen, Temp);

	delete [] Temp;

	return Result;
}

template <>
inline uint32 FCrc::Strihash_DEPRECATED(const int32 DataLen, const UTF8CHAR* Data)
{
	// We can't utilize StringConv.h here due to circular includes, so do the conversion manually

	int32 ConvertedLen = FPlatformString::ConvertedLength<WIDECHAR>(Data, DataLen);

	WIDECHAR* Temp = new WIDECHAR[ConvertedLen];

	WIDECHAR* TempEnd = FPlatformString::Convert(Temp, ConvertedLen, Data, DataLen);
	checkf(TempEnd!=nullptr, TEXT("String conversion unsuccessful"));

	// This doesn't work for strings containing characters outside the BMP, but
	// then neither does the WIDECHAR overload.
	uint32 Result = FCrc::Strihash_DEPRECATED(ConvertedLen, Temp);

	delete [] Temp;

	return Result;
}

/**
 * Gets a non-owning TCHAR pointer from a string type.
 *
 * Can be used generically to get a const TCHAR*, when it is not known if the argument is a TCHAR* or an FString:
 *
 * template <typename T>
 * void LogValue(const T& Val)
 * {
 *     Logf(TEXT("Value: %s"), ToCStr(LexToString(Val)));
 * }
 */
UE_FORCEINLINE_HINT const TCHAR* ToCStr(const TCHAR* Ptr)
{
	return Ptr;
}

/**
 * An implementation of KeyFuncs for sets which hashes string pointers by FCrc::Strihash_DEPRECATED.
 */
template <typename InKeyType, bool bInAllowDuplicateKeys = false>
struct TStringPointerSetKeyFuncs_DEPRECATED
{
	static_assert(TIsCharType_V<std::remove_pointer_t<decltype(ToCStr(std::declval<InKeyType>()))>>, "TStringPointerSetKeyFuncs_DEPRECATED should only be used with keys which character types");

	using KeyType         = InKeyType;
	using KeyInitType     = typename TTypeTraits<InKeyType>::ConstPointerType;
	using ElementInitType = typename TCallTraits<InKeyType>::ParamType;

	static constexpr bool bAllowDuplicateKeys = bInAllowDuplicateKeys;

	static UE_FORCEINLINE_HINT KeyInitType GetSetKey(ElementInitType Element)
	{
		return Element;
	}

	template <typename ComparableKey>
	static UE_FORCEINLINE_HINT bool Matches(KeyInitType A, const ComparableKey& B)
	{
		return A == B;
	}

	template <typename ComparableKey = KeyInitType>
	static UE_FORCEINLINE_HINT uint32 GetKeyHash(ComparableKey Key)
	{
		return FCrc::Strihash_DEPRECATED(ToCStr(Key));
	}
};

/**
 * An implementation of KeyFuncs for maps which hashes string pointers by FCrc::Strihash_DEPRECATED.
 */
template <typename InKeyType, typename InValueType, bool bInAllowDuplicateKeys = false>
struct TStringPointerMapKeyFuncs_DEPRECATED
{
	static_assert(TIsCharType_V<std::remove_pointer_t<decltype(ToCStr(std::declval<InKeyType>()))>>, "TStringPointerMapKeyFuncs_DEPRECATED should only be used with keys which character types");

	using KeyType         = InKeyType;
	using KeyInitType     = typename TTypeTraits<InKeyType>::ConstPointerType;
	using ElementInitType = const TTuple<typename TTypeTraits<InKeyType>::ConstInitType, typename TTypeTraits<InValueType>::ConstInitType>&;

	static constexpr bool bAllowDuplicateKeys = bInAllowDuplicateKeys;

	static UE_FORCEINLINE_HINT KeyInitType GetSetKey(ElementInitType Element)
	{
		return Element.Key;
	}

	template <typename ComparableKey>
	static UE_FORCEINLINE_HINT bool Matches(KeyInitType A, const ComparableKey& B)
	{
		return A == B;
	}

	template <typename ComparableKey = KeyInitType>
	static UE_FORCEINLINE_HINT uint32 GetKeyHash(ComparableKey Key)
	{
		return FCrc::Strihash_DEPRECATED(ToCStr(Key));
	}
};
