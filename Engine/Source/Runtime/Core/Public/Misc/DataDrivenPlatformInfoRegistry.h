// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Misc/Guid.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FConfigFile;

#define DDPI_HAS_EXTENDED_PLATFORMINFO_DATA (WITH_EDITOR || IS_PROGRAM)

enum class EPlatformInfoType : uint8
{
	/** Only return "real" platforms (no groups or fake platforms) */
	TruePlatformsOnly,

	/** All known platforms/groups/etc that have a DDPI.ini file */
	AllPlatformInfos,
};

#if DDPI_HAS_EXTENDED_PLATFORMINFO_DATA


/** Available icon sizes (see FPlatformIconPaths) */
enum class EPlatformIconSize : uint8
{
	/** Normal sized icon (24x24) */
	Normal,

	/** Large sized icon (64x64) */
	Large,

	/** Extra large sized icon (128x128) */
	XLarge,
};



// enum class EPlatformSDKStatus : uint8
// {
// 	/** SDK status is unknown */
// 	Unknown,
// 
// 	/** SDK is installed */
// 	Installed,
// 
// 	/** SDK is not installed */
// 	NotInstalled,
// };

/** Information about where to find the platform icons (for use by FAppStyle) */
struct FPlatformIconPaths
{
	FPlatformIconPaths()
	{
	}

	FPlatformIconPaths(const FString& InIconPath)
		: NormalPath(InIconPath)
		, LargePath(InIconPath)
		, XLargePath(InIconPath)
	{
	}

	FPlatformIconPaths(const FString& InNormalPath, const FString& InLargePath)
		: NormalPath(InNormalPath)
		, LargePath(InLargePath)
		, XLargePath(InLargePath)
	{
	}

	FPlatformIconPaths(const FString& InNormalPath, const FString& InLargePath, const FString& InXLargePath)
		: NormalPath(InNormalPath)
		, LargePath(InLargePath)
		, XLargePath(InXLargePath)
	{
	}

	FName NormalStyleName;
	FString NormalPath;

	FName LargeStyleName;
	FString LargePath;

	FName XLargeStyleName;
	FString XLargePath;
};


/** Information for feature level menu item added by this platform */
struct FPreviewPlatformMenuItem
{
	FName PlatformName;
	FName ShaderFormat;
	FName ShaderPlatformToPreview;
	FName PreviewShaderPlatformName;
	FName PreviewFeatureLevelName;
	FString ActiveIconPath;
	FName ActiveIconName;
	FString InactiveIconPath;
	FName InactiveIconName;
	FText OptionalFriendlyNameOverride;
	FText MenuTooltip;
	FText IconText;
	FName DeviceProfileName;
};

#endif


// Information about a platform loaded from disk
struct FDataDrivenPlatformInfo
{
	// copy of the platform name, same as the Key into GetAllPlatformInfos()
	FName IniPlatformName;

	// cached list of ini parents
	TArray<FString> IniParentChain;

	// is this platform confidential
	bool bIsConfidential = false;

	// some platforms are here just for IniParentChain needs and are not concrete platforms
	bool bIsFakePlatform = false;

	// the name of the ini section to use to load target platform settings (used at runtime and cooktime)
	FString TargetSettingsIniSectionName;

	// list of additional restricted folders
	TArray<FString> AdditionalRestrictedFolders;

	// GUID to represent this platform forever
	FGuid GlobalIdentifier;

	// MemoryFreezing information, matches FPlatformTypeLayoutParameters - defaults are clang, noneditor
	uint32 Freezing_MaxFieldAlignment = 0xffffffff;
	bool Freezing_b32Bit = false;
	bool Freezing_bForce64BitMemoryImagePointers = false;
	bool Freezing_bAlignBases = false;

	// True if this platform has a non-generic gamepad specifically associated with it
	bool bHasDedicatedGamepad = false;

	// True if this platform handles input via standard keyboard layout by default, translates to PC platform
	bool bDefaultInputStandardKeyboard = false;

	// Input-related settings
	bool bInputSupportConfigurable = false;
	FString DefaultInputType = "Gamepad";
	bool bSupportsMouseAndKeyboard = false;
	bool bSupportsGamepad = true;
	bool bCanChangeGamepadType = true;
	bool bSupportsTouch = false;

	// the compression format that this platform wants; overrides game unless bForceUseProjectCompressionFormat
	FString HardwareCompressionFormat;

	/** The ICU data directory name this platform ships (e.g. "icudt64l", "icudt78l"). Empty if unset. */
	FString ICUDataVersion;

	/** Platform name to be used for cooking (note: DOES NOT include the configuration name, eg "Server", "Client", only the platform name) */
	FName OverrideCookPlatformName;

	const FName& GetCookPlatformName() const
	{
		if (!OverrideCookPlatformName.IsNone() && OverrideCookPlatformName.IsValid())
		{
			return OverrideCookPlatformName;
		}
		return IniPlatformName;
	}

	// NOTE: add more settings here (and read them in in the LoadDDPIIniSettings() function in the .cpp)


#if DDPI_HAS_EXTENDED_PLATFORMINFO_DATA

public:


	// setting moved from PlatformInfo::FTargetPlatformInfo


	/** Information about where to find the platform icons (for use by FAppStyle) */
	FPlatformIconPaths IconPaths;

	/** Path under CarefullyRedist for the SDK.  FString so case sensitive platforms don't get messed up by a pre-existing FName of a different casing. */
	FString AutoSDKPath;

	/** Tutorial path for tutorial to install SDK */
	FString SDKTutorial;

	/** An identifier to group similar platforms together, such as "Mobile" and "Console". Used for Per-Platform Override Properties. */
	FName PlatformGroupName;

	/** Submenu name to group similar platforms together in menus, such as "Linux" and "LinuxArm64".  */
	FName PlatformSubMenu;

	/** An identifier that corresponds to UBT's UnrealTargetPlatform enum (and by proxy, FGenericPlatformMisc::GetUBTPlatform()), as well as the directory Binaries are placed under */
	FName UBTPlatformName;
	FString UBTPlatformString;

	/** Whether or not the platform can use Crash Reporter */
	bool bCanUseCrashReporter;

	/** Enabled for use */
	bool bEnabledForUse;

	/** Whether code projects for this platform require the host platform compiler to be installed. Host platforms typically have a SDK status of valid, but they can't necessarily build. */
	bool bUsesHostCompiler;

	/** Whether UAT closes immediately after launching on this platform, or if it sticks around to read output from the running process */
	bool bUATClosesAfterLaunch;

	/** Whether or not this editor/program has compiled in support for this platform (by looking for TargetPlatform style DLLs, without loading them) */
	bool bHasCompiledTargetSupport;



	/** Get the icon name (for FAppStyle) used by the given icon type for this platform */
	FName GetIconStyleName(const EPlatformIconSize InIconSize) const
	{
		switch (InIconSize)
		{
		case EPlatformIconSize::Normal:
			return IconPaths.NormalStyleName;
		case EPlatformIconSize::Large:
			return IconPaths.LargeStyleName;
		case EPlatformIconSize::XLarge:
			return IconPaths.XLargeStyleName;
		default:
			break;
		}
		return NAME_None;
	}

	/** Get the path to the icon on disk (for FAppStyle) for the given icon type for this platform */
	const FString& GetIconPath(const EPlatformIconSize InIconSize) const
	{
		switch (InIconSize)
		{
		case EPlatformIconSize::Normal:
			return IconPaths.NormalPath;
		case EPlatformIconSize::Large:
			return IconPaths.LargePath;
		case EPlatformIconSize::XLarge:
			return IconPaths.XLargePath;
		default:
			break;
		}
		static const FString EmptyString;
		return EmptyString;
	}

private:
	friend struct FDataDrivenPlatformInfoRegistry;

#endif
};


struct FDataDrivenPlatformInfoRegistry
{

	/**
	* Get the global set of data driven platform information
	*/
	static CORE_API const TMap<FName, FDataDrivenPlatformInfo>& GetAllPlatformInfos();

	/**
	 * Gets a set of platform names based on GetAllPlatformInfos, their AdditionalRestrictedFolders
	 * This is not necessarily the same as IniParents, although there is overlap - IniParents come from chaining DDPIs, so those will be in GetAllPlatformInfos already to be checked 
	 *
	 * @param bCheckValid	If true, the result is filtered based on what editor has support compiled for
	 */
	static CORE_API const TArray<FString>& GetPlatformDirectoryNames(bool bCheckValid);

	/**
	 * Gets a set of platform names based on GetAllPlatformInfos, their AdditionalRestrictedFolders, and possibly filtered based on what editor has support compiled for
	 * This is not necessarily the same as IniParents, although there is overlap - IniParents come from chaining DDPIs, so those will be in GetAllPlatformInfos already to be checked 
	 */
	static const TArray<FString>& GetValidPlatformDirectoryNames() { return GetPlatformDirectoryNames(true); }

	/**
	 * Get the data driven platform info for a given platform. If the platform doesn't have any on disk,
	 * this will return a default constructed FConfigDataDrivenPlatformInfo
	 */
	static CORE_API const FDataDrivenPlatformInfo& GetPlatformInfo(const FString& PlatformName);
	static CORE_API const FDataDrivenPlatformInfo& GetPlatformInfo(FName PlatformName);
	static CORE_API const FDataDrivenPlatformInfo& GetPlatformInfo(const char* PlatformName);


	/**
	 * Get sorted platorm infos or names. PlatformType is used to choose between true platforms and platform groups (or other fake platforms)
	 */
	static CORE_API const TArray<FName> GetSortedPlatformNames(EPlatformInfoType PlatformType);
	static CORE_API const TArray<const FDataDrivenPlatformInfo*>& GetSortedPlatformInfos(EPlatformInfoType PlatformType);

	/**
	 * Gets a list of all known confidential platforms (note these are just the platforms you have access to, so, for example PS4 won't be
	 * returned if you are not a PS4 licensee)
	 */
	static CORE_API const TArray<FName>& GetConfidentialPlatforms();

	/**
	 * Returns the number of discovered ini files that can be loaded with LoadDataDrivenIniFile
	 */
	static CORE_API int32 GetNumDataDrivenIniFiles();

	/**
	 * Load the given ini file, and 
	 */
	static CORE_API bool LoadDataDrivenIniFile(int32 Index, FConfigFile& IniFile, FString& PlatformName);


#if DDPI_HAS_EXTENDED_PLATFORMINFO_DATA
	/**
	 * Checks for the existence of compiled modules for a given (usually another, target, platform)
	 * Since there are different types of platform names, it is necessary pass in the type of name
	 */
	enum class EPlatformNameType
	{
		// for instance Win64
		UBT,
		// for instance Windows
		Ini,
		// for instance WindowsClient
		TargetPlatform,
	};
	static CORE_API bool HasCompiledSupportForPlatform(FName PlatformName, EPlatformNameType PlatformNameType);

	/**
	 * Wipes out cached device status for all devices in a platform (or all platforms if PlatformName is empty)
	 */
	static CORE_API void ClearDeviceStatus(FName PlatformName);

	static CORE_API FDataDrivenPlatformInfo& DeviceIdToInfo(FString DeviceId, FString* OutDeviceName = nullptr);

	/**
	 * Retrieve the full list of all known preview platforms from all DDPIs
	*/
	static CORE_API const TArray<struct FPreviewPlatformMenuItem>& GetAllPreviewPlatformMenuItems();
	
	/**
	 * Retrieve the PreviewPlatform MenuItems for ShaderPlatformName
	*/
	static CORE_API const TArray<struct FPreviewPlatformMenuItem>* GetMenuItemForShaderPlatformName(FName ShaderPlatformName);

	/** Option to hide a platform from the user interface at runtime */
	static CORE_API bool IsPlatformHiddenFromUI(FName PlatformName);
	static CORE_API void SetPlatformHiddenFromUI(FName PlatformName);

private:
	/**
	 * Get a modifiable DDPI object, for Turnkey to update it's info
	 */
	friend class FTurkeySupportModule;
	static CORE_API FDataDrivenPlatformInfo& GetMutablePlatformInfo(FName PlatformName);
	static CORE_API TMap<FName, FDataDrivenPlatformInfo>& GetMutablePlatformInfos();

#endif

};

