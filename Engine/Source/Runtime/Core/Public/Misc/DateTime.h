// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/Timespan.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/TypeHash.h"

class FArchive;
class FOutputDevice;
class UObject;


/**
 * Enumerates the days of the week in 7-day calendars.
 */
enum class EDayOfWeek
{
	Monday = 0,
	Tuesday,
	Wednesday,
	Thursday,
	Friday,
	Saturday,
	Sunday
};


/**
 * Enumerates the months of the year in 12-month calendars.
 */
enum class EMonthOfYear
{
	January = 1,
	February,
	March,
	April,
	May,
	June,
	July,
	August,
	September,
	October,
	November,
	December
};


/**
 * Implements a date and time.
 *
 * Values of this type represent dates and times between Midnight 00:00:00, January 1, 0001 and
 * Midnight 23:59:59.9999999, December 31, 9999 in the Gregorian calendar. Internally, the time
 * values are stored in ticks of 0.1 microseconds (= 100 nanoseconds) since January 1, 0001.
 *
 * To retrieve the current local date and time, use the FDateTime.Now() method. To retrieve the
 * current UTC time, use the FDateTime.UtcNow() method instead.
 *
 * This class also provides methods to convert dates and times from and to string representations,
 * calculate the number of days in a given month and year, check for leap years and determine the
 * time of day, day of week and month of year of a given date and time.
 *
 * The companion struct FTimespan is provided for enabling date and time based arithmetic, such as
 * calculating the difference between two dates or adding a certain amount of time to a given date.
 *
 * Ranges of dates and times can be represented by the FDateRange class.
 *
 * @see FDateRange
 * @see FTimespan
 */
struct FDateTime
{
public:

	/** Default constructor (zero initialization). */
	FDateTime()
		: Ticks(0)
	{ }

	/**
	 * Creates and initializes a new instance with the specified number of ticks.
	 *
	 * @param InTicks The ticks representing the date and time.
	 */
	FDateTime(int64 InTicks)
		: Ticks(InTicks)
	{ }

	/**
	 * Creates and initializes a new instance with the specified year, month, day, hour, minute, second and millisecond.
	 *
	 * @param Year The year.
	 * @param Month The month.
	 * @param Day The day.
	 * @param Hour The hour (optional).
	 * @param Minute The minute (optional).
	 * @param Second The second (optional).
	 * @param Millisecond The millisecond (optional).
	 */
	CORE_API FDateTime(int32 Year, int32 Month, int32 Day, int32 Hour = 0, int32 Minute = 0, int32 Second = 0, int32 Millisecond = 0);

public:

	/**
	 * Returns result of adding the given time span to this date.
	 *
	 * @return A date whose value is the sum of this date and the given time span.
	 * @see FTimespan
	 */
	FDateTime operator+(const FTimespan& Other) const
	{
		return FDateTime(Ticks + Other.GetTicks());
	}

	/**
	 * Adds the given time span to this date.
	 *
	 * @return This date.
	 * @see FTimespan
	 */
	FDateTime& operator+=(const FTimespan& Other)
	{
		Ticks += Other.GetTicks();

		return *this;
	}

	/**
	 * Returns time span between this date and the given date.
	 *
	 * @return A time span whose value is the difference of this date and the given date.
	 * @see FTimespan
	 */
	FTimespan operator-(const FDateTime& Other) const
	{
		return FTimespan(Ticks - Other.Ticks);
	}

	/**
	 * Returns result of subtracting the given time span from this date.
	 *
	 * @return A date whose value is the difference of this date and the given time span.
	 * @see FTimespan
	 */
	FDateTime operator-(const FTimespan& Other) const
	{
		return FDateTime(Ticks - Other.GetTicks());
	}

	/**
	 * Subtracts the given time span from this date.
	 *
	 * @return This date.
	 * @see FTimespan
	 */
	FDateTime& operator-=(const FTimespan& Other)
	{
		Ticks -= Other.GetTicks();

		return *this;
	}

	/**
	 * Compares this date with the given date for equality.
	 *
	 * @param Other The date to compare with.
	 * @return true if the dates are equal, false otherwise.
	 */
	bool operator==(const FDateTime& Other) const
	{
		return (Ticks == Other.Ticks);
	}

	/**
	 * Compares this date with the given date for inequality.
	 *
	 * @param Other The date to compare with.
	 * @return true if the dates are not equal, false otherwise.
	 */
	bool operator!=(const FDateTime& Other) const
	{
		return (Ticks != Other.Ticks);
	}

	/**
	 * Checks whether this date is greater than the given date.
	 *
	 * @param Other The date to compare with.
	 * @return true if this date is greater, false otherwise.
	 */
	bool operator>(const FDateTime& Other) const
	{
		return (Ticks > Other.Ticks);
	}

	/**
	 * Checks whether this date is greater than or equal to the date span.
	 *
	 * @param Other The date to compare with.
	 * @return true if this date is greater or equal, false otherwise.
	 */
	bool operator>=(const FDateTime& Other) const
	{
		return (Ticks >= Other.Ticks);
	}

	/**
	 * Checks whether this date is less than the given date.
	 *
	 * @param Other The date to compare with.
	 * @return true if this date is less, false otherwise.
	 */
	bool operator<(const FDateTime& Other) const
	{
		return (Ticks < Other.Ticks);
	}

	/**
	 * Checks whether this date is less than or equal to the given date.
	 *
	 * @param Other The date to compare with.
	 * @return true if this date is less or equal, false otherwise.
	 */
	bool operator<=(const FDateTime& Other) const
	{
		return (Ticks <= Other.Ticks);
	}

public:

	/**
	 * Exports the date and time value to a string.
	 *
	 * @param ValueStr Will hold the string value.
	 * @param DefaultValue The default value.
	 * @param Parent Not used.
	 * @param PortFlags Not used.
	 * @param ExportRootScope Not used.
	 * @return true on success, false otherwise.
	 * @see ImportTextItem
	 */
	CORE_API bool ExportTextItem(FString& ValueStr, FDateTime const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;

	/**
	 * Gets the date part of this date.
	 *
	 * The time part is truncated and becomes 00:00:00.000.
	 *
	 * @return A FDateTime object containing the date.
	 */
	FDateTime GetDate() const
	{
		return FDateTime(Ticks - (Ticks % ETimespan::TicksPerDay));
	}

	/**
	 * Gets the date components of this date.
	 *
	 * @param OutYear Will contain the year.
	 * @param OutMonth Will contain the number of the month (1-12).
	 * @param OutDay Will contain the number of the day (1-31).
	 */
	CORE_API void GetDate(int32& OutYear, int32& OutMonth, int32& OutDay) const;

	/**
	 * Gets this date's day part (1 to 31).
	 *
	 * @return Day of the month.
	 * @see GetHour, GetHour12, GetMillisecond, GetMinute, GetMonth, GetSecond, GetYear
	 */
	CORE_API int32 GetDay() const;

	/**
	 * Calculates this date's day of the week (Sunday - Saturday).
	 *
	 * @return The week day.
	 * @see GetDayOfYear, GetMonthOfYear, GetTimeOfDay
	 */
	CORE_API EDayOfWeek GetDayOfWeek() const;

	/**
	 * Gets this date's day of the year.
	 *
	 * @return The day of year.
	 * @see GetDayOfWeek, GetMonthOfYear, GetTimeOfDay
	 */
	CORE_API int32 GetDayOfYear() const;

	/**
	 * Gets this date's hour part in 24-hour clock format (0 to 23).
	 *
	 * @return The hour.
	 * @see GetDay, GetDayOfWeek, GetDayOfYear, GetHour12, GetMillisecond, GetMinute, GetMonth, GetSecond, GetYear
	 */
	int32 GetHour() const
	{
		return (int32)((Ticks / ETimespan::TicksPerHour) % 24);
	}

	/**
	 * Gets this date's hour part in 12-hour clock format (1 to 12).
	 *
	 * @return The hour in AM/PM format.
	 * @see GetDay, GetHour, GetMillisecond, GetMinute, GetMonth, GetSecond, GetYear
	 */
	CORE_API int32 GetHour12() const;

	/**
	 * Returns the Julian Day for this date.
	 *
	 * The Julian Day is the number of days since the inception of the Julian calendar at noon on
	 * Monday, January 1, 4713 B.C.E. The minimum Julian Day that can be represented in FDateTime is
	 * 1721425.5, which corresponds to Monday, January 1, 0001 in the Gregorian calendar.
	 *
	 * @return Julian Day.
	 * @see FromJulianDay, GetModifiedJulianDay
	 */
	double GetJulianDay() const
	{
		return 1721425.5 + double(Ticks / ETimespan::TicksPerDay) + GetTimeOfDay().GetTotalDays();
	}

	/**
	 * Returns the Modified Julian day.
	 *
	 * The Modified Julian Day is calculated by subtracting 2400000.5, which corresponds to midnight UTC on
	 * November 17, 1858 in the Gregorian calendar.
	 *
	 * @return Modified Julian Day
	 * @see GetJulianDay
	 */
	double GetModifiedJulianDay() const
	{
		return (GetJulianDay() - 2400000.5);
	}

	/**
	 * Gets this date's millisecond part (0 to 999).
	 *
	 * @return The millisecond.
	 * @see GetDay, GetHour, GetHour12, GetMinute, GetMonth, GetSecond, GetYear
	 */
	int32 GetMillisecond() const
	{
		return (int32)((Ticks / ETimespan::TicksPerMillisecond) % 1000);
	}

	/**
	 * Gets this date's minute part (0 to 59).
	 *
	 * @return The minute.
	 * @see GetDay, GetHour, GetHour12, GetMillisecond, GetMonth, GetSecond, GetYear
	 */
	int32 GetMinute() const
	{
		return (int32)((Ticks / ETimespan::TicksPerMinute) % 60);
	}

	/**
	 * Gets this date's the month part (1 to 12).
	 *
	 * @return The month.
	 * @see GetDay, GetHour, GetHour12, GetMillisecond, GetMinute, GetSecond, GetYear
	 */
	CORE_API int32 GetMonth() const;

	/**
	 * Gets the date's month of the year (January to December).
	 *
	 * @return Month of year.
	 * @see GetDayOfWeek, GetDayOfYear, GetTimeOfDay
	 */
	EMonthOfYear GetMonthOfYear() const
	{
		return static_cast<EMonthOfYear>(GetMonth());
	}

	/**
	 * Gets this date's second part.
	 *
	 * @return The second.
	 * @see GetDay, GetHour, GetHour12, GetMillisecond, GetMinute, GetMonth, GetYear
	 */
	int32 GetSecond() const
	{
		return (int32)((Ticks / ETimespan::TicksPerSecond) % 60);
	}

	/**
	 * Gets this date's representation as number of ticks.
	 *
	 * @return Number of ticks since midnight, January 1, 0001.
	 */
	int64 GetTicks() const
	{
		return Ticks;
	}

	/**
	 * Gets the time elapsed since midnight of this date.
	 *
	 * @return Time of day since midnight.
	 * @see GetDayOfWeek, GetDayOfYear, GetMonthOfYear
	 */
	FTimespan GetTimeOfDay() const
	{
		return FTimespan(Ticks % ETimespan::TicksPerDay);
	}

	/**
	 * Gets this date's year part.
	 *
	 * @return The year.
	 * @see GetDay, GetHour, GetHour12, GetMillisecond, GetMinute, GetMonth, GetSecond
	 */
	CORE_API int32 GetYear() const;

	/**
	 * Imports a date and time value from a text buffer.
	 *
	 * @param Buffer The text buffer to import from.
	 * @param PortFlags Not used.
	 * @param Parent Not used.
	 * @param ErrorText The output device for error logging.
	 * @return true on success, false otherwise.
	 * @see ExportTextItem
	 */
	CORE_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);

	/**
	 * Gets whether this date's time is in the afternoon.
	 *
	 * @return true if it is in the afternoon, false otherwise.
	 * @see IsMorning
	 */
	bool IsAfternoon() const
	{
		return (GetHour() >= 12);
	}

	/**
	 * Gets whether this date's time is in the morning.
	 *
	 * @return true if it is in the morning, false otherwise.
	 * @see IsAfternoon
	 */
	bool IsMorning() const
	{
		return (GetHour() < 12);
	}

	/**
	 * Serializes this date and time from or into the specified archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @return true on success, false otherwise.
	 */
	CORE_API bool Serialize(FArchive& Ar);

	/**
	* Serializes this date and time for network transmission
	*/
	CORE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/**
	 * Returns the RFC 1123 string representation of the FDateTime.
	 *
	 * The resulting string assumes that the FDateTime is in UTC.
	 * 
	 * @return String representation.
	 * @see ParseHttpDate, ToIso8601, ToString
	 */
	CORE_API FString ToHttpDate() const;

	/**
	 * Returns the ISO-8601 string representation of the FDateTime.
	 *
	 * The resulting string assumes that the FDateTime is in UTC.
	 * 
	 * @return String representation.
	 * @see ParseIso8601, ToHttpDate, ToString
	 */
	CORE_API FString ToIso8601() const;

	/**
	 * Returns the string representation of this date using a default format.
	 *
	 * The returned string has the following format:
	 *		yyyy.mm.dd-hh.mm.ss
	 *
	 * @return String representation.
	 * @see Parse, ToIso8601
	 */
	CORE_API FString ToString() const;

	/**
	 * Returns the string representation of this date.
	 *
	 * ToString uses a non-standard format syntax (see below). If you need strftime-like syntax, then use ToFormattedString:
	 *		%a - am or pm
	 *		%A - AM or PM
	 *		%d - Day, 01-31
	 *		%D - Day of the Year, 001-366
	 *		%m - Month, 01-12
	 *		%y - Year, YY
	 *		%Y - Year, YYYY
	 *		%h - 12h Hour, 01-12
	 *		%H - 24h Hour, 00-23
	 *		%M - Minute, 00-59
	 *		%S - Second, 00-60
	 *		%s - Millisecond, 000-999
	 * 
	 * @param Format The format of the returned string.
	 * @return String representation.
	 * @see Parse, ToIso8601
	 */
	CORE_API FString ToString(const TCHAR* Format) const;
	CORE_API void ToString(const TCHAR* Format, FStringBuilderBase& Result) const;

	/**
	 * Returns the string representation of this date.
	 *
	 * Uses strftime-like syntax:
	 *		%a - Weekday, eg) Sun
	 *		%A - Weekday, eg) Sunday
	 *		%w - Weekday, 0-6 (Sunday is 0)
	 *		%y - Year, YY
	 *		%Y - Year, YYYY
	 *		%b - Month, eg) Jan
	 *		%B - Month, eg) January
	 *		%m - Month, 01-12
	 *		%d - Day, 01-31
	 *		%e - Day, 1-31
	 *		%l - 12h Hour, 1-12
	 *		%I - 12h Hour, 01-12
	 *		%H - 24h Hour, 00-23
	 *		%M - Minute, 00-59
	 *		%S - Second, 00-60
	 *		%p - AM or PM
	 *		%P - am or pm
	 *		%j - Day of the Year, 001-366
	 * 
	 * @param Format The format of the returned string.
	 * @return String representation.
	 */
	CORE_API FString ToFormattedString(const TCHAR* Format) const;

	/**
	 * Returns this date as the number of seconds since the Unix Epoch (January 1st of 1970).
	 *
	 * @return Time of day.
	 * @see FromUnixTimestamp
	 */
	int64 ToUnixTimestamp() const
	{
		return (Ticks - FDateTime(1970, 1, 1).Ticks) / ETimespan::TicksPerSecond;
	}

	/**
	 * Returns this date as the number of seconds since the Unix Epoch (January 1st of 1970).
	 *
	 * @return Time of day.
	 * @see FromUnixTimestamp
	 */
	double ToUnixTimestampDecimal() const
	{
		return double(Ticks - FDateTime(1970, 1, 1).Ticks) / ETimespan::TicksPerSecond;
	}

public:

	/**
	 * Gets the number of days in the year and month.
	 *
	 * @param Year The year.
	 * @param Month The month.
	 * @return The number of days
	 * @see DaysInYear
	 */
	static CORE_API int32 DaysInMonth(int32 Year, int32 Month);

	/**
	 * Gets the number of days in the given year.
	 *
	 * @param Year The year.
	 * @return The number of days.
	 * @see DaysInMonth
	 */
	static CORE_API int32 DaysInYear(int32 Year);

	/**
	 * Returns the proleptic Gregorian date for the given Julian Day.
	 *
	 * @param JulianDay The Julian Day.
	 * @return Gregorian date and time.
	 * @see GetJulianDay
	 */
	static FDateTime FromJulianDay(double JulianDay)
	{
		return FDateTime((int64)((JulianDay - 1721425.5) * ETimespan::TicksPerDay));
	}

	/**
	 * Returns the date from Unix time (seconds from midnight 1970-01-01)
	 *
	 * @param UnixTime Unix time (seconds from midnight 1970-01-01)
	 * @return Gregorian date and time.
	 * @see ToUnixTimestamp
	 */
	static FDateTime FromUnixTimestamp(int64 UnixTime)
	{
		return FDateTime(1970, 1, 1) + FTimespan(UnixTime * ETimespan::TicksPerSecond);
	}

	/**
	 * Returns the date from Unix time (seconds from midnight 1970-01-01)
	 *
	 * @param UnixTime Unix time (seconds from midnight 1970-01-01)
	 * @return Gregorian date and time.
	 * @see ToUnixTimestampDecimal
	 */
	static FDateTime FromUnixTimestampDecimal(double UnixTime)
	{
		return FDateTime(1970, 1, 1) + FTimespan(int64(UnixTime * ETimespan::TicksPerSecond));
	}

	/**
	 * Checks whether the given year is a leap year.
	 *
	 * A leap year is a year containing one additional day in order to keep the calendar synchronized
	 * with the astronomical year. All years divisible by 4, but not divisible by 100 - except if they
	 * are also divisible by 400 - are leap years.
	 *
	 * @param Year The year to check.
	 * @return true if the year is a leap year, false otherwise.
	 */
	static CORE_API bool IsLeapYear(int32 Year);

	/**
	 * Returns the maximum date value.
	 *
	 * The maximum date value is December 31, 9999, 23:59:59.9999999.
	 *
	 * @see MinValue
	 */
	static FDateTime MaxValue()
	{
		return FDateTime(3652059 * ETimespan::TicksPerDay - 1);
	}

	/**
	 * Returns the minimum date value.
	 *
	 * The minimum date value is January 1, 0001, 00:00:00.0.
	 *
	 * @see MaxValue
	 */
	static FDateTime MinValue()
	{
		return FDateTime(0);
	}

	/**
	 * Gets the local date and time on this computer.
	 *
	 * This method takes into account the local computer's time zone and daylight saving
	 * settings. For time zone independent time comparisons, and when comparing times
	 * between different computers, please use UtcNow() instead.
	 *
	 * @return Current date and time.
	 * @see Today, UtcNow
	 */
	static CORE_API FDateTime Now();

	/**
	 * Converts a string to a date and time.
	 *
	 * Currently, the string must be in the format written by either FDateTime.ToString() or
	 * FTimeStamp.TimestampToFString(). Other formats are not supported at this time.
	 *
	 * @param DateTimeString The string to convert.
	 * @param OutDateTime Will contain the parsed date and time.
	 * @return true if the string was converted successfully, false otherwise.
	 * @see ParseHttpDate, ParseIso8601, ToString
	 */
	static CORE_API bool Parse(const FString& DateTimeString, FDateTime& OutDateTime);

	/**
	 * Parses a date string in HTTP-date format (rfc1123-date | rfc850-date | asctime-date)
	 * https://www.w3.org/Protocols/rfc2616/rfc2616-sec3.html#sec3.3.1
	 *
 	 * HTTP-date    = rfc1123-date | rfc850-date | asctime-date
	 * rfc1123-date = wkday "," SP date1 SP time SP "GMT"
	 * rfc850-date  = weekday "," SP date2 SP time SP "GMT"
	 * asctime-date = wkday SP date3 SP time SP 4DIGIT
	 * date1        = 2DIGIT SP month SP 4DIGIT ; day month year (e.g., 02 Jun 1982)
	 * date2        = 2DIGIT "-" month "-" 2DIGIT ; day-month-year (e.g., 02-Jun-82)
	 * date3        = month SP (2DIGIT | (SP 1DIGIT)) ; month day (e.g., Jun  2)
	 * time         = 2DIGIT ":" 2DIGIT ":" 2DIGIT ; 00:00:00 - 23:59:59
	 * wkday        = "Mon" | "Tue" | "Wed" | "Thu" | "Fri" | "Sat" | "Sun"
	 * weekday      = "Monday" | "Tuesday" | "Wednesday" | "Thursday" | "Friday" | "Saturday" | "Sunday"
	 * month        = "Jan" | "Feb" | "Mar" | "Apr" | "May" | "Jun" | "Jul" | "Aug" | "Sep" | "Oct" | "Nov" | "Dec"
	 *
	 * @param HttpDate The string to be parsed
	 * @param OutDateTime FDateTime object (assumes UTC) corresponding to the input string.
	 * @return true if the string was converted successfully, false otherwise.
	 * @see Parse, ToHttpDate, ParseIso8601
	 */
	static CORE_API bool ParseHttpDate(const FString& HttpDate, FDateTime& OutDateTime);

	/**
	 * Parses a date string in ISO-8601 format.
	 * 
	 * @param DateTimeString The string to be parsed
	 * @param OutDateTime FDateTime object (in UTC) corresponding to the input string (which may have been in any timezone).
	 * @return true if the string was converted successfully, false otherwise.
	 * @see Parse, ParseHttpDate, ToIso8601
	 */
	static CORE_API bool ParseIso8601(const TCHAR* DateTimeString, FDateTime& OutDateTime);

	/**
	 * Gets the local date on this computer.
	 *
	 * The time component is set to 00:00:00
	 *
	 * @return Current date.
	 * @see Now, UtcNow
	 */
	static FDateTime Today()
	{
		return Now().GetDate();
	}

	/**
	 * Gets the UTC date and time on this computer.
	 *
	 * This method returns the Coordinated Universal Time (UTC), which does not take the
	 * local computer's time zone and daylight savings settings into account. It should be
	 * used when comparing dates and times that should be independent of the user's locale.
	 * To get the date and time in the current locale, use Now() instead.
	 *
	 * This method will use an estimate if USE_ESTIMATED_UTCNOW is 1. To calculate the 
	 * estimate it will initialize a base reference time and keep track of the offset from 
	 * that time with FPlatformTime::Cycles64().  
	 * This is appropriate for platforms whose implementations of FPlatformTime::UtcTime 
	 * are expensive.  To use, enable bUseEstimatedUtcNow in TargetRules.  The rebase time
	 * is stored in time.EstimatedUtcNowRebaseTimeSeconds (default setting is 600 seconds).
	 *
	 * @return Current date and time.
	 * @see Now
	 */
	static CORE_API FDateTime UtcNow();

	/**
	 * Validates the given components of a date and time value.
	 *
	 * The allow ranges for the components are:
	 *		Year: 1 - 9999
	 *		Month: 1 - 12
	 *		Day: 1 - DaysInMonth(Month)
	 *		Hour: 0 - 23
	 *		Minute: 0 - 59
	 *		Second: 0 - 59
	 *		Millisecond: 0 - 999
	 *
	 * @return true if the components are valid, false otherwise.
	 */
	static CORE_API bool Validate(int32 Year, int32 Month, int32 Day, int32 Hour, int32 Minute, int32 Second, int32 Millisecond);

public:

	/**
	 * Serializes the given date and time from or into the specified archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param DateTime The date and time value to serialize.
	 * @return The archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, FDateTime& DateTime)
	{
		return Ar << DateTime.Ticks;
	}

	/**
	 * Serializes the given date and time from or into the specified structured archive slot.
	 *
	 * @param Slot The structured archive slot to serialize from or into.
	 * @param DateTime The date and time value to serialize.
	 */
	friend void operator<<(FStructuredArchive::FSlot Slot, FDateTime& DateTime)
	{
		return Slot << DateTime.Ticks;
	}

	/**
	 * Gets the hash for the specified date and time.
	 *
	 * @param DateTime The date and time to get the hash for.
	 * @return Hash value.
	 */
	friend uint32 GetTypeHash(const FDateTime& DateTime)
	{
		return GetTypeHash(DateTime.Ticks);
	}

protected:

	/** Holds the days per month in a non-leap year. */
	static const int32 DaysPerMonth[];

	/** Holds the cumulative days per month in a non-leap year. */
	static const int32 DaysToMonth[];

	/** Holds the long and short day names. */
	static const TCHAR* ShortDayNames[];
	static const TCHAR* LongDayNames[];

	/** Holds the long and short month names. */
	static const TCHAR* ShortMonthNames[];
	static const TCHAR* LongMonthNames[];

private:
	friend struct Z_Construct_UScriptStruct_FDateTime_Statics;

private:

	/** Holds the ticks in 100 nanoseconds resolution since January 1, 0001 A.D. */
	int64 Ticks;
};
