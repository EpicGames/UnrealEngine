// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Math/MathFwd.h"
#include "Misc/Timespan.h"

struct FColor;
struct FLinearColor;

//
//	FDefaultValueHelper
//

class FDefaultValueHelper
{
	/** Something like TEXT macro for chars is needed */
	static TCHAR TS(const TCHAR* Sign);

	static bool IsWhitespace(TCHAR Char);

	/** 
	 *	advances Pos to first non-whitespace symbol
	 *	returns if some non-Whitespace sign remains in string 
	 */
	static bool Trim(int32& Pos, const FString& Source);

	/** returns address of the first char in the string */
	static const TCHAR* StartOf(const FString& Source);

	/** returns address of the last (empty) char in the string */
	static const TCHAR* EndOf(const FString& Source);

	/** 
	 *	advances Pos to first non-whitespace symbol
	 *	returns if some non-Whitespace sign remains in string 
	 */
	static bool Trim(const TCHAR* & Start, const TCHAR* End );

	/**
	 *	@param Start - address of the first symbol of the checked string
	 *  @param End - address of the first symbol after of the checked string (also address of the terminating empty symbol)
	 */
	static bool IsStringValidFloat(const TCHAR* Start, const TCHAR* End);

	/** return if the string can be parse to an int. OutBase is the base of the integer (8, 10, 16)*/
	static bool IsStringValidInteger(const TCHAR* Start, const TCHAR* End, int32& OutBase);
	static bool IsStringValidInteger(const TCHAR* Start, const TCHAR* End);

public:

	static CORE_API FString GetUnqualifiedEnumValue(const FString& Source);

	static CORE_API bool HasWhitespaces(const FString& Source);

	static CORE_API FString RemoveWhitespaces(const FString& Source);

	/** Shell parameters list: " TypeName ( A a, B b ) " -> "A a, B b" */
	static CORE_API bool GetParameters(const FString& Source, const FString& TypeName, FString& OutForm);

	/** returns if given strings are equal, ignores initial and final white spaces in Source */
	static CORE_API bool Is(const FString& Source, const TCHAR* CompareStr);

	/**   
	 * source forms:	TypeName( TEXT ("ABC") ), TEXT("ABC"), TypeName("ABC"), "ABC"
	 * output form:		ABC
	 */
	static CORE_API bool StringFromCppString(const FString& Source, const FString& TypeName, FString& OutForm);

	/*
	 *	Following functions accept c++ style representations of numbers.
	 *  e.g. 13.5e-2f for float or -0x123 for int
	 */
	static CORE_API bool IsStringValidInteger(const FString& Source);

	static CORE_API bool IsStringValidFloat(const FString& Source);

	/** accepted form: " %f, %f, %f" */
	static CORE_API bool IsStringValidVector(const FString& Source);

	/** accepted form: " %f, %f, %f" */
	static CORE_API bool IsStringValidRotator(const FString& Source);

	/** accepted form: " %f, %f, %f " or " %f, %f, %f, %f " (alpha is optional)  */
	static CORE_API bool IsStringValidLinearColor(const FString& Source);


	/**
	 * Converts a string into a int32.
	 *
	 * @param Source the input string to try to convert
	 * @param OutVal the output integer
	 *
	 * @return true if the conversion happened, false otherwise
	 */
	UE_REWRITE static bool ParseInt(const FString& Source, int32& OutVal)
	{ 
		return ParseInt(*Source, OutVal); 
	}
	static CORE_API bool ParseInt(const TCHAR* Source, int32& OutVal);

	/**
	 * Converts a string into a bool.
	 * Accepted form: "true" or "false" or "1" or "0"
	 *
	 * @param Source the input string to try to convert
	 * @param OutVal the output boolean
	 *
	 * @return true if the conversion happened, false otherwise
	 */
	static CORE_API bool ParseBool(const FString& Source, bool& OutVal);

	/**
	 * Converts a string into a int64.
	 *
	 * @param Source the input string to try to convert
	 * @param OutVal the output integer
	 *
	 * @return true if the conversion happened, false otherwise
	 */
	static CORE_API bool ParseInt64(const FString& Source, int64& OutVal);

	/**
	 * Converts a string into a float.
	 *
	 * @param Source the input string to try to convert
	 * @param OutVal the output float
	 *
	 * @return true if the conversion happened, false otherwise
	 */
	static CORE_API bool ParseFloat(const FString& Source, float& OutVal);

	/**
	 * Converts a string into a double.
	 *
	 * @param Source the input string to try to convert
	 * @param OutVal the output double
	 *
	 * @return true if the conversion happened, false otherwise
	 */
	static CORE_API bool ParseDouble(const FString& Source, double& OutVal);

	/**
	 * Converts a string into a FVector. Accepted form: " %f, %f, %f "
	 *
	 * @param Source the input string to try to convert
	 * @param OutVal the output vector
	 *
	 * @return true if the conversion happened, false otherwise
	 */
	static CORE_API bool ParseVector(const FString& Source, FVector3f& OutVal);
	static CORE_API bool ParseVector(const FString& Source, FVector3d& OutVal);

	/**
	 * Converts a string into a FVector. Accepted form: " %f, %f "
	 *
	 * @param Source the input string to try to convert
	 * @param OutVal the output vector2D
	 *
	 * @return true if the conversion happened, false otherwise
	 */
	static CORE_API bool ParseVector2D(const FString& Source, FVector2f& OutVal);
	static CORE_API bool ParseVector2D(const FString& Source, FVector2d& OutVal);

	/**
	* Converts a string into a FVector4. Accepted form: " %f, %f, %f, %f "
	*
	* @param Source the input string to try to convert
	* @param OutVal the output vector4
	*
	* @return true if the conversion happened, false otherwise
	*/
	static CORE_API bool ParseVector4(const FString& Source, FVector4f& OutVal);
	static CORE_API bool ParseVector4(const FString& Source, FVector4d& OutVal);

	/**
	 * Converts a string into a FRotator. Accepted form: " %f, %f, %f "
	 *
	 * @param Source the input string to try to convert
	 * @param OutVal the output rotator
	 *
	 * @return true if the conversion happened, false otherwise
	 */
	static CORE_API bool ParseRotator(const FString& Source, FRotator3f& OutVal);
	static CORE_API bool ParseRotator(const FString& Source, FRotator3d& OutVal);

	/**
	 * Converts a string into a FLinearColor. 
	 * Accepted forms: " %f, %f, %f " or " %f, %f, %f, %f " (alpha is optional).
	 *
	 * @param Source the input string to try to convert
	 * @param OutVal the output color
	 *
	 * @return true if the conversion happened, false otherwise
	 */
	static CORE_API bool ParseLinearColor(const FString& Source, FLinearColor& OutVal);

	/**
	 * Converts a string into a FLinearColor. 
	 * Accepted forms: " %d, %d, %d " or " %d, %d, %d, %d " (alpha is optional).
	 *
	 * @param Source the input string to try to convert
	 * @param OutVal the output color
	 *
	 * @return true if the conversion happened, false otherwise
	 */
	static CORE_API bool ParseColor(const FString& Source, FColor& OutVal);
};
