// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PreprocessorHelpers.h"
#include "Templates/Identity.h"

namespace UE::Core::Private
{
	template <auto Storage, auto PtrToMember>
	struct TPrivateAccess
	{
		TPrivateAccess()
		{
			*Storage = PtrToMember;
		}

		static TPrivateAccess Instance;
	};

	template <auto Storage, auto PtrToMember>
	TPrivateAccess<Storage, PtrToMember> TPrivateAccess<Storage, PtrToMember>::Instance;
}

// A way to get a pointer-to-member of private members of a class without explicit friendship.
// It can be used for both data members and member functions.
//
// Use of this macro for any purpose is at the user's own risk and is not supported.
//
// Example:
//
// struct FPrivateStuff
// {
//     explicit FPrivateStuff(int32 InVal)
//     {
//         Val = InVal;
//     }
//
// private:
//     int32 Val;
//
//     void LogVal() const
//     {
//         UE_LOGF(LogTemp, Log, "Val: %d", Val);
//     }
// };
//
// // These should be defined at global scope
// UE_DEFINE_PRIVATE_MEMBER_PTR(int32, GPrivateStuffValPtr, FPrivateStuff, Val);
// UE_DEFINE_PRIVATE_MEMBER_PTR(void() const, GPrivateStuffLogVal, FPrivateStuff, LogVal);
//
// FPrivateStuff Stuff(5);
//
// (Stuff.*GPrivateStuffLogVal)(); // Logs: "Val: 5"
// Stuff.*GPrivateStuffValPtr = 7;
// (Stuff.*GPrivateStuffLogVal)(); // Logs: "Val: 7"
//
#define UE_DEFINE_PRIVATE_MEMBER_PTR(Type, Name, Class, Member) \
    TIdentity_T<UE_REMOVE_OPTIONAL_PARENS(Type)> UE_REMOVE_OPTIONAL_PARENS(Class)::* Name; \
    template struct UE::Core::Private::TPrivateAccess<&Name, &UE_REMOVE_OPTIONAL_PARENS(Class)::Member>
