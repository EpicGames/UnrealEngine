// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

template <typename FuncType> class TFunction;

enum class EDelayedRegisterRunPhase : uint8
{
	StartOfEnginePreInit,
	FileSystemReady,
	TaskGraphSystemReady,
	StatSystemReady,
	IniSystemReady,
	EarliestPossiblePluginsLoaded,
	PreRHIInit,
	ShaderTypesReady,
	PreObjectSystemReady,
	ObjectSystemReady,
	DeviceProfileManagerReady,
	EndOfEngineInit,

	NumRunOncePhases, // Phases before this are run once

	LiveCodingReload,

	NumPhases,
};

struct FDelayedAutoRegisterHelper
{

	CORE_API FDelayedAutoRegisterHelper(EDelayedRegisterRunPhase RunPhase, TFunction<void()> RegistrationFunction, const bool bRerunOnLiveCodingReload = false);

	/** Overload that provides the phase being registered at the time the function is called.  */
	CORE_API FDelayedAutoRegisterHelper(EDelayedRegisterRunPhase RunPhase, TFunction<void(const EDelayedRegisterRunPhase)> RegistrationFunction, const bool bRerunOnLiveCodingReload = false);

	static CORE_API void RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase RunPhase);
};

