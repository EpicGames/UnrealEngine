// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"
#include "Misc/IFilter.h"

/**
 *	A generic filter designed to call a predicate provided on construction to filter
 */
template< typename ItemType >
class TDelegateFilter
	: public IFilter< ItemType >, public TSharedFromThis< TDelegateFilter< ItemType > >
{
public:

	DECLARE_DELEGATE_RetVal_OneParam( bool, FPredicate, ItemType );

	/** 
	 *	TDelegateFilter Constructor
	 *	
	 * @param	InPredicate		A required delegate called to determine if an Item passes the filter
	 */
	TDelegateFilter( FPredicate InPredicate ) 
		: Predicate( InPredicate )
	{
		check( Predicate.IsBound() );
	}

	DECLARE_DERIVED_EVENT( TDelegateFilter, IFilter<ItemType>::FChangedEvent, FChangedEvent );
	virtual FChangedEvent& OnChanged() override { return ChangedEvent; }

	/** 
	 * Returns whether the specified Item passes the Filter's restrictions 
	 *
	 *	@param	InItem	The Item is check 
	 *	@return			Whether the specified Item passed the filter
	 */
	virtual bool PassesFilter( ItemType InItem ) const override
	{
		return Predicate.Execute( InItem );
	}

	/** Broadcasts the OnChanged event for this filter */
	void BroadcastChanged()
	{
		ChangedEvent.Broadcast();
	}

private:

	/** The delegate called to determine if an Item passes the filter */
	FPredicate Predicate;

	/**	Fires whenever the filter changes */
	FChangedEvent ChangedEvent;
};
