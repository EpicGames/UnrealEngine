// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// C5105: macro expansion producing 'defined' has undefined behavior
// however, in all compilers we use, the behavior is as desired
#pragma warning(disable : 5105) 

// Used to allow control flow when direct header compiling to avoid issues with headers
// that typically expect to be included after another header
#ifndef UE_DIRECT_HEADER_COMPILING
	#define UE_DIRECT_HEADER_COMPILING(id) defined __COMPILING_##id
#endif