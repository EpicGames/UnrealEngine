// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Delegates/MulticastDelegateBase.h"
#include "HAL/CriticalSection.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

typedef TMap<FString, FString> FEmbeddedCommunicationMap;

// wraps parameters and a completion delegate
struct FEmbeddedCallParamsHelper
{
	// the command for this call. something like "console" would be a command for the engine to run a console command
	FString Command;
	
	// a map of arbitrary key-value string pairs.
	FEmbeddedCommunicationMap Parameters;

	// a delegate to call back on the other end when the command is completed. if this is set, it is expected it will be called
	// but at least one handler of this
	TFunction<void(const FEmbeddedCommunicationMap&, FString)> OnCompleteDelegate;
};

class FEmbeddedDelegates
{
public:

	// delegate for calling between native wrapper app and embedded ue
	DECLARE_MULTICAST_DELEGATE_OneParam(FEmbeddedCommunicationParamsDelegate, const FEmbeddedCallParamsHelper&);

	// calling in from native wrapper to engine
	static CORE_API FEmbeddedCommunicationParamsDelegate& GetNativeToEmbeddedParamsDelegateForSubsystem(FName SubsystemName);

	// calling out from engine to native wrapper
	static CORE_API FEmbeddedCommunicationParamsDelegate& GetEmbeddedToNativeParamsDelegateForSubsystem(FName SubsystemName);

	// returns true if NativeToEmbedded delegate for subsystem exists
	static CORE_API bool IsEmbeddedSubsystemAvailable(FName SubsystemName);
	
	// FTSTicker-like delegate, to bind things to be ticked at a regular interval while the game thread is otherwise asleep.
	static CORE_API FSimpleMulticastDelegate SleepTickDelegate;

	// get/set an object by name, thread safe
	static CORE_API void SetNamedObject(const FString& Name, void* Object);
	static CORE_API void* GetNamedObject(const FString& Name);
	
private:

	// This class is only for namespace use (like FCoreDelegates)
	FEmbeddedDelegates() {}

	// the per-subsystem delegates
	static TMap<FName, FEmbeddedCommunicationParamsDelegate> NativeToEmbeddedDelegateMap;
	static TMap<FName, FEmbeddedCommunicationParamsDelegate> EmbeddedToNativeDelegateMap;

	// named object registry, with thread protection
	static FCriticalSection NamedObjectRegistryLock;
	static TMap<FString, void*> NamedObjectRegistry;
};


class FEmbeddedCommunication
{
public:

	// called early in UE lifecycle - RunOnGameThread can be called before this is called
	static CORE_API void Init();

	// force some ticking to happen - used to process messages during otherwise blocking operations like boot
	static CORE_API void ForceTick(int ID, float MinTimeSlice=0.1f, float MaxTimeSlice=0.5f);

	// queue up a function to call on game thread
	static CORE_API void RunOnGameThread(int Priority, TFunction<void()> Lambda);

	// wake up the game thread to process something put onto the game thread
	static CORE_API void WakeGameThread();

	// called from game thread to pull off
	static CORE_API bool TickGameThread(float DeltaTime);
	
	// tell UE to stay awake (or allow it to sleep when nothing to do). Repeated calls with the same Requester are
	// allowed, but bNeedsRendering must match
	static CORE_API void KeepAwake(FName Requester, bool bNeedsRendering);
	static CORE_API void AllowSleep(FName Requester);
	
	static CORE_API void UELogFatal(const TCHAR* String);
	static CORE_API void UELogError(const TCHAR* String);
	static CORE_API void UELogWarning(const TCHAR* String);
	static CORE_API void UELogDisplay(const TCHAR* String);
	static CORE_API void UELogLog(const TCHAR* String);
	static CORE_API void UELogVerbose(const TCHAR* String);

	static CORE_API bool IsAwakeForTicking();
	static CORE_API bool IsAwakeForRendering();
	
	static CORE_API FString GetDebugInfo();
};

// RAII for keep awake functionality
// This may seem a bit over engineered, but it needs to have move semantics so that any aggregate that
// includes it doesn't lose move semantics.
class FEmbeddedKeepAwake
{
public:
	// tell UE to stay awake (or allow it to sleep when nothing to do). Repeated calls with the same Requester are
	// allowed, but bNeedsRendering must match
	FEmbeddedKeepAwake(FName InRequester, bool bInNeedsRendering)
		: Requester(InRequester)
		, bNeedsRendering(bInNeedsRendering)
	{
		FEmbeddedCommunication::KeepAwake(Requester, bNeedsRendering);
	}

	FEmbeddedKeepAwake(const FEmbeddedKeepAwake& Other)
		: Requester(Other.Requester)
		, bNeedsRendering(Other.bNeedsRendering)
		, bIsValid(Other.bIsValid)
	{
		if (bIsValid)
		{
			FEmbeddedCommunication::KeepAwake(Requester, bNeedsRendering);
		}
	}

	FEmbeddedKeepAwake(FEmbeddedKeepAwake&& Other)
	{
		Requester = Other.Requester;
		bNeedsRendering = Other.bNeedsRendering;
		bIsValid = Other.bIsValid;

		Other.Requester = NAME_None;
		Other.bNeedsRendering = false;
		Other.bIsValid = false;
	}

	~FEmbeddedKeepAwake()
	{
		if (bIsValid)
		{
			FEmbeddedCommunication::AllowSleep(Requester);
		}
	}

	FEmbeddedKeepAwake& operator=(const FEmbeddedKeepAwake& Other) &
	{
		bool bOldIsValid = bIsValid;
		FName OldRequester = Requester;

		Requester = Other.Requester;
		bNeedsRendering = Other.bNeedsRendering;
		bIsValid = Other.bIsValid;
		if (bIsValid)
		{
			FEmbeddedCommunication::KeepAwake(Requester, bNeedsRendering);
		}

		if (bOldIsValid)
		{
			FEmbeddedCommunication::AllowSleep(OldRequester);
		}

		return *this;
	}

	FEmbeddedKeepAwake& operator=(FEmbeddedKeepAwake&& Other) &
	{
		if (bIsValid)
		{
			FEmbeddedCommunication::AllowSleep(Requester);
		}

		Requester = Other.Requester;
		bNeedsRendering = Other.bNeedsRendering;
		bIsValid = Other.bIsValid;

		Other.Requester = NAME_None;
		Other.bNeedsRendering = false;
		Other.bIsValid = false;
		return *this;
	}

	bool GetNeedsRendering() const { return bNeedsRendering; }
	FName GetRequester() const { return Requester;  }

private:
	FName Requester;
	bool bNeedsRendering = false;
	bool bIsValid = true;
};

