// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"

class FEngineBuildSettings
{
public:
	/**
	 * @return True if the build was gotten from perforce
	 */
	static CORE_API bool IsPerforceBuild();

	/**
	 * @return True if the build is for internal projects only
	 */
	static CORE_API bool IsInternalBuild();

	/**
	 * @return True if the current installation is a source distribution.
	 */
	static CORE_API bool IsSourceDistribution();

	/**
	 * @return True if a given engine distribution contains source (as opposed to, say, Launcher builds)
	 */
	static CORE_API bool IsSourceDistribution(const FString& RootDir);
};
