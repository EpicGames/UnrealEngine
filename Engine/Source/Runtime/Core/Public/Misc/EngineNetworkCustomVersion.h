// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

struct FEngineNetworkCustomVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// Original replay versions from ENetworkVersionHistory
		Initial = 1,
		ReplayBackwardsCompat = 2,				// Bump version to get rid of older replays before backwards compat was turned on officially
		MaxActorChannelsCustomization = 3,		// Bump version because serialization of the actor channels changed
		RepCmdChecksumRemovePrintf = 4,			// Bump version since the way FRepLayoutCmd::CompatibleChecksum was calculated changed due to an optimization
		NewActorOverrideLevel = 5,				// Bump version since a level reference was added to the new actor information
		ChannelNames = 6,						// Bump version since channel type is now an fname
		ChannelCloseReason = 7,					// Bump version to serialize a channel close reason in bunches instead of bDormant
		AcksIncludedInHeader = 8,				// Bump version since acks are now sent as part of the header
		NetExportSerialization = 9,				// Bump version due to serialization change to FNetFieldExport
		NetExportSerializeFix = 10,				// Bump version to fix net field export name serialization 
		FastArrayDeltaStruct = 11,				// Bump version to allow fast array serialization, delta struct serialization.
		FixEnumSerialization = 12,				// Bump version to fix enum net serialization issues.
		OptionallyQuantizeSpawnInfo = 13,		// Bump version to conditionally disable quantization for Scale, Location, and Velocity when spawning network actors.
		JitterInHeader = 14,					// Bump version since we added jitter clock time to packet headers and removed remote saturation
		ClassNetCacheFullName = 15,				// Bump version to use full paths in GetNetFieldExportGroupForClassNetCache
		ReplayDormancy = 16,					// Bump version to support dormancy properly in replays
		EnumSerializationCompat = 17,			// Bump version to include enum bits required for serialization into compat checksums, as well as unify enum and byte property enum serialization
		SubObjectOuterChain = 18,				// Bump version to support subobject outer chains matching on client and server
		HitResultInstanceHandle = 19,			// Bump version to support FHitResult change of Actor to HitObjectHandle. This change was made in CL 14369221 but a net version wasn't added at the time.
		InterfacePropertySerialization = 20,	// Bump version to support net serialization of FInterfaceProperty
		MontagePlayInstIdSerialization = 21,	// Bump version to support net serialization of FGameplayAbilityRepAnimMontage, addition of PlayInstanceId and removal of bForcePlayBit
		SerializeDoubleVectorsAsDoubles = 22,	// Bump version to support net serialization of double vector types
		PackedVectorLWCSupport = 23,			// Bump version to support quantized LWC FVector net serialization
		PawnRemoteViewPitch = 24,				// Bump version to support serialization changes to RemoteViewPitch
		RepMoveServerFrameAndHandle = 25,		// Bump version to support serialization changes to RepMove so we can get the serverframe and physics handle associated with the object
		Ver21AndViewPitchOnly_DONOTUSE = 26,	// Bump version to support up to history 21 + PawnRemoteViewPitch.  DO NOT USE!!!
		Placeholder = 27,                       // Bump version to a placeholder.  This version is the same as RepMoveServerFrameAndHandle
		RuntimeFeaturesCompatibility = 28,		// Bump version to add network runtime feature compatibility test to handshake (hello/upgrade) control messages
		SoftObjectPtrNetGuids = 29,				// Bump version to support replicating SoftObjectPtrs by NetGuid instead of raw strings.
		SubObjectDestroyFlag = 30,				// Bump version to support subobject destruction message flags
		GameStateReplicatedTimeAsDouble = 31,	// Bump version to support AGameStateBase::ReplicatedWorldTimeSeconds as double instead of float.
		CustomVersions = 32,                    // Bump version to switch to using custom versions
		DynamicMontageSerialization = 33,		// Bump version to support dynamic montage serialization in the Gameplay Ability System
		PredictionKeyBaseNotReplicated = 34,	// Bump version to stop FPredictionKey::Base from being replicated (it was unused).
		RepMoveOptionalAcceleration = 35,		// Bump version to support serialization changes to RepMove for optional Acceleration
		CustomExports = 36,						// Bump version to support CustomExports (such as NetTokens used for supporting "Dynamic Replication" of GameplayTags -- see FGameplayTag::NetSerialize_Packed)		// -----<new versions can be added above this line>-------------------------------------------------
		MontagePlayCountSerialization = 37,		// Bump version to support serialization changes for dynamic montage play count 
		RemoteObjectReferences = 38,			// Bump version to support serialization changes for remote object references 
		BeaconNetIDVariant = 39,				// Bump version for online beacon's BeaconAssignGUID control message switching to use FNetIDVariant
		JoinNoPawn = 40,						// Bump version to support NMT_JoinNoPawn and NMT_JoinNoPawnSplit control messages.
		ClientHandshakeId = 41,					// Bump version to support serialization of APlayerController::ClientHandeshakeId.
		PawnRemoteViewPitchTo16Bit = 42,		// Bump version to support higher precision PawnRemoteViewPitch
		CloseChildConnection = 43,				// Bump version to support closing child connections with NMT_CloseChildConnection.
		ExplicitAckHistorySeq = 44,				// Bump version to add explicit sequence for ack history.

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid Guid;

	FEngineNetworkCustomVersion() = delete;
};

struct FGameNetworkCustomVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid Guid;

	FGameNetworkCustomVersion() = delete;
};
