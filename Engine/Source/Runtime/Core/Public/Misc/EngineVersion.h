// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EngineVersionBase.h"
#include "Containers/UnrealString.h"
#include "Serialization/StructuredArchive.h"

/** Utility functions. */
class FEngineVersion : public FEngineVersionBase
{
public:

	/** Empty constructor. Initializes the version to 0.0.0-0. */
	FEngineVersion() = default;

	/** Constructs a version from the given components. */
	CORE_API FEngineVersion(uint16 InMajor, uint16 InMinor, uint16 InPatch, uint32 InChangelist, const FString &InBranch);

	/** Sets the version to the given values. */
	CORE_API void Set(uint16 InMajor, uint16 InMinor, uint16 InPatch, uint32 InChangelist, const FString &InBranch);

	/** Clears the object. */
	CORE_API void Empty();

	/** Checks whether this engine version is an exact match for another engine version */
	CORE_API bool ExactMatch(const FEngineVersion& Other) const;

	/** Checks compatibility with another version object. */
	CORE_API bool IsCompatibleWith(const FEngineVersionBase &Other) const;

	/** Generates a version string */
	CORE_API FString ToString(EVersionComponent LastComponent = EVersionComponent::Branch) const;

	/** Parses a version object from a string. Returns true on success. */
	static CORE_API bool Parse(const FString &Text, FEngineVersion &OutVersion);

	/** Gets the current engine version */
	static CORE_API const FEngineVersion& Current();

	/** Gets the earliest version which this engine maintains strict API and package compatibility with */
	static CORE_API const FEngineVersion& CompatibleWith();

	/** Clears the current and compatible-with engine versions */
	static CORE_API void TearDown();

	/** Serialization functions */
	friend CORE_API void operator<<(class FArchive &Ar, FEngineVersion &Version);
	friend CORE_API void operator<<(FStructuredArchive::FSlot Slot, FEngineVersion &Version);

	/** Returns the branch name corresponding to this version. */
	const FString GetBranch() const
	{
		return Branch.Replace( TEXT( "+" ), TEXT( "/" ) );
	}

	CORE_API const FString& GetBranchDescriptor() const;
		

private:

	/** Branch name. */
	FString Branch;
};

