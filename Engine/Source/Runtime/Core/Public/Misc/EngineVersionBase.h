// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/** Enum for the components of a version string. */
enum class EVersionComponent
{
	Major,					///< Major version increments introduce breaking API changes.
	Minor,					///< Minor version increments add additional functionality without breaking existing APIs.
	Patch,					///< Patch version increments fix existing functionality without changing the API.
	Changelist,				///< The pre-release field adds additional versioning through a series of comparable dotted strings or numbers.
	Branch,					///<
};


/** Components of a version string. */
enum class EVersionComparison
{
	Neither,
	First,
	Second,
};


/** Base class for the EngineVersion class. Holds basic version numbers. */
class FEngineVersionBase
{
public:

	/** Empty constructor. Initializes the version to 0.0.0-0. */
	FEngineVersionBase() = default;

	/** Constructs a version from the given components. */
	CORE_API FEngineVersionBase(uint16 InMajor, uint16 InMinor, uint16 InPatch = 0, uint32 InChangelist = 0);

	/** Returns the changelist number corresponding to this version. */
	CORE_API uint32 GetChangelist() const;	

	/** Returns the Major version number corresponding to this version. */
	UE_FORCEINLINE_HINT uint16 GetMajor() const
	{
		return Major;
	}

	/** Returns the Minor version number corresponding to this version. */
	UE_FORCEINLINE_HINT uint16 GetMinor() const
	{
		return Minor;
	}

	/** Returns the Patch version number corresponding to this version. */
	UE_FORCEINLINE_HINT uint16 GetPatch() const
	{
		return Patch;
	}

	/** Checks if the changelist number represents licensee changelist number. */
	CORE_API bool IsLicenseeVersion() const;

	/** Returns whether the current version is empty. */
	CORE_API bool IsEmpty() const;

	/** Returns whether the engine version has a changelist component. */
	CORE_API bool HasChangelist() const; 

	/** Returns the newest of two versions, and the component at which they differ */
	static CORE_API EVersionComparison GetNewest(const FEngineVersionBase &First, const FEngineVersionBase &Second, EVersionComponent *OutComponent);

	/** Encodes a licensee changelist number (by setting the top bit) */
	static CORE_API uint32 EncodeLicenseeChangelist(uint32 Changelist);

protected:

	/** Major version number. */
	uint16 Major = 0;

	/** Minor version number. */
	uint16 Minor = 0;

	/** Patch version number. */
	uint16 Patch = 0;

	/** Changelist number. This is used to arbitrate when Major/Minor/Patch version numbers match. Use GetChangelist() instead of using this member directly. */
	uint32 Changelist = 0;
};
