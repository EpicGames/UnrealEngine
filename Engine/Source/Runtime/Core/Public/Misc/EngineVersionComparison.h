// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Runtime/Launch/Resources/Version.h"

// Helper for UE_VERSION_NEWER_THAN and UE_VERSION_OLDER_THAN
#define UE_GREATER_SORT(Value, ValueToBeGreaterThan, TieBreaker) \
	(((Value) > (ValueToBeGreaterThan)) || (((Value) == (ValueToBeGreaterThan)) && (TieBreaker)))

// Version comparison macro that is defined to true if the UE version is newer or equal than MajorVer.MinorVer.PatchVer 
// and false otherwise
// (a typical use is for backward compatible code)
#define UE_VERSION_NEWER_THAN_OR_EQUAL(MajorVersion, MinorVersion, PatchVersion)\
	UE_GREATER_SORT(ENGINE_MAJOR_VERSION, MajorVersion, UE_GREATER_SORT(ENGINE_MINOR_VERSION, MinorVersion, UE_GREATER_SORT(ENGINE_PATCH_VERSION, PatchVersion, true)))

// Version comparison macro that is defined to true if the UE version is newer than MajorVer.MinorVer.PatchVer and false otherwise
// (a typical use is to alert integrators to revisit this code when upgrading to a new engine version)
#define UE_VERSION_NEWER_THAN(MajorVersion, MinorVersion, PatchVersion) \
	UE_GREATER_SORT(ENGINE_MAJOR_VERSION, MajorVersion, UE_GREATER_SORT(ENGINE_MINOR_VERSION, MinorVersion, UE_GREATER_SORT(ENGINE_PATCH_VERSION, PatchVersion, false)))

// Version comparison macro that is defined to true if the UE version is older than MajorVer.MinorVer.PatchVer and false otherwise
// (use when making code that needs to be compatible with older engine versions)
#define UE_VERSION_OLDER_THAN(MajorVersion, MinorVersion, PatchVersion) \
	UE_GREATER_SORT(MajorVersion, ENGINE_MAJOR_VERSION, UE_GREATER_SORT(MinorVersion, ENGINE_MINOR_VERSION, UE_GREATER_SORT(PatchVersion, ENGINE_PATCH_VERSION, false)))
