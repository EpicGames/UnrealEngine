// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

// Defines all bitwise operators for enum classes so it can be (mostly) used as a regular flags enum
#define ENUM_CLASS_FLAGS(Enum) \
	UE_REWRITE constexpr Enum& operator|=(Enum& Lhs, Enum Rhs) { return Lhs = (Enum)((__underlying_type(Enum))Lhs | (__underlying_type(Enum))Rhs); } \
	UE_REWRITE constexpr Enum& operator&=(Enum& Lhs, Enum Rhs) { return Lhs = (Enum)((__underlying_type(Enum))Lhs & (__underlying_type(Enum))Rhs); } \
	UE_REWRITE constexpr Enum& operator^=(Enum& Lhs, Enum Rhs) { return Lhs = (Enum)((__underlying_type(Enum))Lhs ^ (__underlying_type(Enum))Rhs); } \
	UE_REWRITE constexpr Enum  operator| (Enum  Lhs, Enum Rhs) { return (Enum)((__underlying_type(Enum))Lhs | (__underlying_type(Enum))Rhs); } \
	UE_REWRITE constexpr Enum  operator& (Enum  Lhs, Enum Rhs) { return (Enum)((__underlying_type(Enum))Lhs & (__underlying_type(Enum))Rhs); } \
	UE_REWRITE constexpr Enum  operator^ (Enum  Lhs, Enum Rhs) { return (Enum)((__underlying_type(Enum))Lhs ^ (__underlying_type(Enum))Rhs); } \
	UE_REWRITE constexpr bool  operator! (Enum  E)             { return !(__underlying_type(Enum))E; } \
	UE_REWRITE constexpr Enum  operator~ (Enum  E)             { return (Enum)~(__underlying_type(Enum))E; }

// Friends all bitwise operators for enum classes so the definition can be kept private / protected.
#define FRIEND_ENUM_CLASS_FLAGS(Enum) \
	friend constexpr Enum& operator|=(Enum& Lhs, Enum Rhs); \
	friend constexpr Enum& operator&=(Enum& Lhs, Enum Rhs); \
	friend constexpr Enum& operator^=(Enum& Lhs, Enum Rhs); \
	friend constexpr Enum  operator| (Enum  Lhs, Enum Rhs); \
	friend constexpr Enum  operator& (Enum  Lhs, Enum Rhs); \
	friend constexpr Enum  operator^ (Enum  Lhs, Enum Rhs); \
	friend constexpr bool  operator! (Enum  E); \
	friend constexpr Enum  operator~ (Enum  E);

template<typename Enum>
UE_REWRITE constexpr bool EnumHasAllFlags(Enum Flags, Enum Contains)
{
	using UnderlyingType = __underlying_type(Enum);
	return ((UnderlyingType)Flags & (UnderlyingType)Contains) == (UnderlyingType)Contains;
}

template<typename Enum>
UE_REWRITE constexpr bool EnumHasAnyFlags(Enum Flags, Enum Contains)
{
	using UnderlyingType = __underlying_type(Enum);
	return ((UnderlyingType)Flags & (UnderlyingType)Contains) != 0;
}

/**
 * Check if the Flags value contains only the flags specified by the Contains argument.
 * It is not required that any of the flags in Contains be set in Flags and Flags may be
 * any combination of the flags specified by Contains (including k=0). But if Flags 
 * contains any flag not in Contains, this function will return false.
 * This function may also be understood as !EnumContainsAnyFlags(Flags, ~Contains).
 *
 * @param Flags    The value to check.
 * @param Contains The flags to check the value against.
 *
 * @return true if Flags consists of only the flags specified by Contains. false otherwise.
 * @note   Returns true if Flags is empty.
 */
template <typename Enum>
UE_REWRITE constexpr bool EnumOnlyContainsFlags(Enum Flags, Enum Contains)
{
	return EnumHasAllFlags(Contains, Flags);
}

/**
 * Check if Flags has one and only one flag set.
 * This can also be thought of as a check for a power-of-2 value.
 *
 * @param Flags The value to check.
 *
 * @return true if Flags has only one flag set. false otherwise.
 */
template <typename Enum>
UE_REWRITE constexpr bool EnumHasOneFlag(Enum Flags)
{
	using UnderlyingType = __underlying_type(Enum);
	return ((UnderlyingType)Flags != 0) && (((UnderlyingType)Flags & ((UnderlyingType)Flags - 1)) == 0);
}

/**
 * Check if Flags has one and only one of the flags specified in OneOfFlags set
 *
 * @param Flags       The value to check.
 * @param OneOfFlags  The flags to check the value for.
 *
 * @return true if Flags has one and only one of the flags in OneOfFlags set. false otherwise.
 */
template <typename Enum>
UE_REWRITE constexpr bool EnumHasAnyOneFlag(Enum Flags, Enum OneOfFlags)
{
	using UnderlyingType = __underlying_type(Enum);
	return EnumHasOneFlag((Enum)((UnderlyingType)Flags & (UnderlyingType)OneOfFlags));
}

template<typename Enum>
UE_REWRITE constexpr void EnumAddFlags(Enum& Flags, Enum FlagsToAdd)
{
	using UnderlyingType = __underlying_type(Enum);
	Flags = (Enum)((UnderlyingType)Flags | (UnderlyingType)FlagsToAdd);
}

template<typename Enum>
UE_REWRITE constexpr void EnumRemoveFlags(Enum& Flags, Enum FlagsToRemove)
{
	using UnderlyingType = __underlying_type(Enum);
	Flags = (Enum)((UnderlyingType)Flags & ~(UnderlyingType)FlagsToRemove);
}

template <typename Enum>
UE_REWRITE constexpr Enum EnumLowestSetFlag(Enum Flags)
{
	using UnderlyingType = __underlying_type(Enum);
#ifdef _MSC_VER
	#pragma warning(push)
	#pragma warning(disable : 4146) // unary minus operator applied to unsigned type, result still unsigned
#endif
	return Flags & (Enum)(-(UnderlyingType)Flags);
#ifdef _MSC_VER
	#pragma warning(pop)
#endif
}

template <typename Enum>
UE_REWRITE constexpr Enum EnumRemoveLowestSetFlag(Enum Flags)
{
	using UnderlyingType = __underlying_type(Enum);
	return (Enum)((UnderlyingType)Flags & ((UnderlyingType)Flags - 1));
}

template <typename Enum>
UE_NODEBUG constexpr int EnumNumSetFlags(Enum Flags)
{
	using UnderlyingType = __underlying_type(Enum);

	int Result = 0;
	UnderlyingType Int = (UnderlyingType)Flags;
	while (Int != 0)
	{
		++Result;
		Int &= Int - 1;
	}
	return Result;
}

// Converts an enumerator value to the same value in its underlying type.
template <typename Enum>
UE_NODEBUG constexpr auto EnumToUnderlyingType(Enum E)
{
	return (__underlying_type(Enum))E;
}
