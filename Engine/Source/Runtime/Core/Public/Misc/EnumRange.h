// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include <type_traits>

/**
 * Defines a contiguous enum range containing Count values, starting from zero:
 *
 * Example:
 *
 * enum class ECountedThing
 * {
 *     First,
 *     Second,
 *     Third,
 *
 *     Count
 * };
 *
 * // Defines iteration over ECountedThing to be: First, Second, Third
 * ENUM_RANGE_BY_COUNT(ECountedThing, ECountedThing::Count)
 */
#define ENUM_RANGE_BY_COUNT(EnumType, Count) ENUM_RANGE_BY_FIRST_AND_LAST(EnumType, 0, (std::underlying_type_t<EnumType>)(Count) - 1)


/**
 * Defines a contiguous enum range with specific start and end values:
 *
 * Example:
 *
 * enum class EDoubleEndedThing
 * {
 *     Invalid,
 *
 *     First,
 *     Second,
 *     Third,
 *
 *     Count
 * };
 *
 * // Defines iteration over EDoubleEndedThing to be: First, Second, Third
 * ENUM_RANGE_BY_FIRST_AND_LAST(EDoubleEndedThing, EDoubleEndedThing::First, EDoubleEndedThing::Third)
 */
#define ENUM_RANGE_BY_FIRST_AND_LAST(EnumType, First, Last) \
	template <> \
	struct NEnumRangePrivate::TEnumRangeTraits<EnumType> \
	{ \
		enum { RangeType = 0 }; \
		static constexpr std::underlying_type_t<EnumType> Begin = (std::underlying_type_t<EnumType>)(First); \
		static constexpr std::underlying_type_t<EnumType> End   = (std::underlying_type_t<EnumType>)(Last) + 1; \
	};


/**
 * Defines a non-contiguous enum range with specific individual values:
 *
 * Example:
 *
 * enum class ERandomValuesThing
 * {
 *     First  = 2,
 *     Second = 3,
 *     Third  = 5,
 *     Fourth = 7,
 *     Fifth  = 11
 * };
 *
 * // Defines iteration over ERandomValuesThing to be: First, Second, Third, Fourth, Fifth
 * ENUM_RANGE_BY_VALUES(ERandomValuesThing, ERandomValuesThing::First, ERandomValuesThing::Second, ERandomValuesThing::Third, ERandomValuesThing::Fourth, ERandomValuesThing::Fifth)
 */
#define ENUM_RANGE_BY_VALUES(EnumType, ...) \
	template <> \
	struct NEnumRangePrivate::TEnumRangeTraits<EnumType> \
	{ \
		enum { RangeType = 1 }; \
		template <typename Dummy> \
		static const EnumType* GetPointer(bool bLast) \
		{ \
			static constexpr EnumType Values[] = { __VA_ARGS__ }; \
			return bLast ? Values + sizeof(Values) / sizeof(EnumType) : Values; \
		} \
	};


namespace NEnumRangePrivate
{
	template <typename EnumType>
	struct TEnumRangeTraits
	{
		enum { RangeType = -1 };
	};

	template <typename EnumType, int32 RangeType>
	struct TEnumRange_Impl
	{
		static_assert(sizeof(EnumType) == 0, "Unknown enum type - use one of the ENUM_RANGE macros to define iteration semantics for your enum type.");
	};

	template <typename EnumType>
	struct TEnumContiguousIterator
	{
		using IntType = std::underlying_type_t<EnumType>;

		UE_FORCEINLINE_HINT explicit TEnumContiguousIterator(IntType InValue)
			: Value(InValue)
		{
		}

		inline TEnumContiguousIterator& operator++()
		{
			++Value;
			return *this;
		}

		UE_FORCEINLINE_HINT EnumType operator*() const
		{
			return (EnumType)Value;
		}

	private:
		IntType Value;

		UE_FORCEINLINE_HINT friend bool operator!=(const TEnumContiguousIterator& Lhs, const TEnumContiguousIterator& Rhs)
		{
			return Lhs.Value != Rhs.Value;
		}
	};

	template <typename EnumType>
	struct TEnumValueArrayIterator
	{
		UE_FORCEINLINE_HINT explicit TEnumValueArrayIterator(const EnumType* InPtr)
			: Ptr(InPtr)
		{
		}

		inline TEnumValueArrayIterator& operator++()
		{
			++Ptr;
			return *this;
		}

		UE_FORCEINLINE_HINT EnumType operator*() const
		{
			return *Ptr;
		}

	private:
		const EnumType* Ptr;

		UE_FORCEINLINE_HINT friend bool operator!=(const TEnumValueArrayIterator& Lhs, const TEnumValueArrayIterator& Rhs)
		{
			return Lhs.Ptr != Rhs.Ptr;
		}
	};

	template <typename EnumType>
	struct TEnumRange_Impl<EnumType, 0>
	{
	    using IntType = typename TEnumContiguousIterator<EnumType>::IntType;

		inline explicit TEnumRange_Impl()
			: BeginValue(TEnumRangeTraits<EnumType>::Begin)
			, EndValue(TEnumRangeTraits<EnumType>::End)
		{
		}

		/**
		 * Note that this is a closed range, so it's not currently possible (or valid) to iterate over zero elements.
		 * By specifying the same first and last value, it will iterate over that single value.
		 */
		inline explicit TEnumRange_Impl(const EnumType InFirstValue, const EnumType InLastValue)
			: BeginValue(static_cast<IntType>(InFirstValue))
			, EndValue(static_cast<IntType>(InLastValue) + 1)
		{
			checkf(
				BeginValue < EndValue
				&& BeginValue >= TEnumRangeTraits<EnumType>::Begin
				&& EndValue <= TEnumRangeTraits<EnumType>::End,
				TEXT("The provided first and last enum values must be within the available range specified in the ENUM_RANGE macro."));
		}

		TEnumContiguousIterator<EnumType> begin() const { return TEnumContiguousIterator<EnumType>(BeginValue); }
		TEnumContiguousIterator<EnumType> end()   const { return TEnumContiguousIterator<EnumType>(EndValue  ); }

	private:
        const IntType BeginValue;
        const IntType EndValue;
	};

	template <typename EnumType>
	struct TEnumRange_Impl<EnumType, 1>
	{
		UE_FORCEINLINE_HINT explicit TEnumRange_Impl() = default;

		UE_FORCEINLINE_HINT explicit TEnumRange_Impl(const EnumType InFirstValue, const EnumType InLastValue) = delete;

		TEnumValueArrayIterator<EnumType> begin() const { return TEnumValueArrayIterator<EnumType>(TEnumRangeTraits<EnumType>::template GetPointer<void>(false)); }
		TEnumValueArrayIterator<EnumType> end  () const { return TEnumValueArrayIterator<EnumType>(TEnumRangeTraits<EnumType>::template GetPointer<void>(true )); }
	};
}

namespace UE::EnumFlags::Private
{
	template <typename EnumType>
	struct TIterator
	{
		using IntType = std::underlying_type_t<EnumType>;

		UE_FORCEINLINE_HINT explicit TIterator(EnumType InFlags)
			: Flags(IntType(InFlags))
		{
		}

		inline TIterator& operator++()
		{
			const IntType PoppedBit = Flags & (~Flags + 1);
			Flags ^= PoppedBit;
			return *this;
		}

		inline EnumType operator*() const
		{
			const IntType Result = Flags & (~Flags + 1); 
			return (EnumType)Result;
		}

	private:
		IntType Flags;

		UE_FORCEINLINE_HINT friend bool operator!=(const TIterator& Lhs, const TIterator& Rhs)
		{
			return Lhs.Flags != Rhs.Flags;
		}
	};

	template <typename EnumType>
	struct TRange
	{
		explicit TRange(EnumType InFlags) : Flags(InFlags) {}

		Private::TIterator<EnumType> begin() const { return Private::TIterator<EnumType>(Flags); }
		Private::TIterator<EnumType> end() const { return Private::TIterator<EnumType>(EnumType(0)); }

	private:
		EnumType Flags;
	};
} // namespace UE::EnumFlags::Private

/**
 * Range type for iterating over enum values.  Enums should define themselves as iterable by specifying
 * one of the ENUM_RANGE_* macros.
 *
 * Example:
 *
 * for (ECountedThing Val : TEnumRange<ECountedThing>())
 * {
 *     ...
 * }
 **/
template <typename EnumType>
struct TEnumRange : NEnumRangePrivate::TEnumRange_Impl<EnumType, NEnumRangePrivate::TEnumRangeTraits<EnumType>::RangeType>
{
	using Super = NEnumRangePrivate::TEnumRange_Impl<EnumType, NEnumRangePrivate::TEnumRangeTraits<EnumType>::RangeType>;

	using Super::Super;
};

/**
 * Make a range for iterating over set flags in a flags enum.
 *
 * Example:
 *
 * EFlagThing Flags = EFlagThing::A | EFlagThing::B;
 * for (EFlagThing Flag : MakeFlagsRange(Flags))
 * {
 *     // Loop is run twice, once with Flag = EFlagThing::A, once with Flag = EFlagThing::B
 *     ...
 * }
 **/
template <typename EnumType>
UE::EnumFlags::Private::TRange<EnumType> MakeFlagsRange(EnumType Flags)
{
	return UE::EnumFlags::Private::TRange<EnumType>(Flags);
}