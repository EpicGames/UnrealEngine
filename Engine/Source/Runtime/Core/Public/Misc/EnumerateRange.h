// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UnrealTemplate.h"

#include <iterator>
#include <type_traits>

template <typename ElementType, typename SizeType = int32>
struct TEnumerateRef
{
	TEnumerateRef(ElementType& InRef, SizeType InIndex)
		: Ref  (InRef)
		, Index(InIndex)
	{
	}

	SizeType GetIndex() const
	{
		return this->Index;
	}

	ElementType& operator*() const
	{
		return this->Ref;
	}

	ElementType* operator->() const
	{
		return &this->Ref;
	}

	operator TEnumerateRef<const ElementType>() const
	{
		return { Ref, Index };
	}

private:
	ElementType& Ref;
	SizeType     Index;
};

template <typename ElementType, typename SizeType = int32>
using TConstEnumerateRef = TEnumerateRef<const ElementType, SizeType>;

namespace UE::Core::Private
{
	template <typename IteratorType, typename SizeType>
	struct TEnumerateIter
	{
		IteratorType Iterator;
		SizeType     Index = 0;

		auto operator*() const -> TEnumerateRef<std::remove_reference_t<decltype(*std::declval<const IteratorType&>())>, SizeType>
		{
			return { *this->Iterator, this->Index };
		}

		template <typename EndIteratorType>
		bool operator!=(EndIteratorType End) const
		{
			return this->Iterator != End;
		}

		void operator++()
		{
			++this->Iterator;
			++this->Index;
		}
	};
}

template <typename RangeType, typename SizeType>
struct TEnumerateRange
{
	RangeType Range;

	auto begin() const -> UE::Core::Private::TEnumerateIter<decltype(std::begin(Range)), SizeType>
	{
		return { std::begin(Range) };
	}

	auto end() const
	{
		return std::end(Range);
	}
};

/**
 * Allows iterating over a range while also keeping track of the current iteration index.
 *
 * Usage:
 *
 * for (TEnumerateRef<T> Elem : EnumerateRange(Container))
 * {
 *     Elem->MemberFunctionOfT();
 *     FunctionTakingT(*Elem);
 *     int32 Index = Elem.GetIndex();
 * }
 *
 * TEnumerateRef<const T> or TConstEnumerateRef<T> can be used for const iteration.
 *
 * Iterating over larger containers like TArray64 will require an explicit SizeType
 * parameter: TEnumerateRef<T, int64>
 */
template <typename RangeType>
auto EnumerateRange(RangeType&& Range) -> TEnumerateRange<RangeType, decltype(GetNum(Range))>
{
	return { (RangeType&&)Range };
}
template <typename T, SIZE_T N>
TEnumerateRange<T(&)[N], int32> EnumerateRange(T(&Range)[N])
{
	static_assert(N <= MAX_int32, "Array size is not supported by Enumerate");
	return { Range };
}
