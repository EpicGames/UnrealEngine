// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"

class FOutputDevice;
class UWorld;

#ifndef UE_ALLOW_EXEC_COMMANDS
	#if UE_BUILD_SHIPPING && !WITH_EDITOR
		#define UE_ALLOW_EXEC_COMMANDS UE_ALLOW_EXEC_COMMANDS_IN_SHIPPING
	#else
		#define UE_ALLOW_EXEC_COMMANDS 1
	#endif
#endif

#ifndef UE_ALLOW_EXEC_DEV
	#define UE_ALLOW_EXEC_DEV !UE_BUILD_SHIPPING && UE_ALLOW_EXEC_COMMANDS
#endif

#ifndef UE_ALLOW_EXEC_EDITOR
	#define UE_ALLOW_EXEC_EDITOR WITH_EDITOR && UE_ALLOW_EXEC_COMMANDS
#endif

// Any object that is capable of taking commands.
class FExec
{
public:
	CORE_API virtual ~FExec();

#if UE_ALLOW_EXEC_COMMANDS

	/**
	 * Exec handler
	 *
	 * @param	InWorld	World context
	 * @param	Cmd		Command to parse
	 * @param	Ar		Output device to log to
	 *
	 * @return	true if command was handled, false otherwise
	 */
	CORE_API virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar );
#else // UE_ALLOW_EXEC_COMMANDS

	/**
	 * final override of Exec that asserts if called
	 */
	CORE_API virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) final;
	
	/**
	 * final override of Exec to replace overrides where a default value for Ar is provided
	 */
	CORE_API virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd ) final;

#endif // !UE_ALLOW_EXEC_COMMANDS

protected:
	/** Implementation of Exec that is called on all targets where UE_ALLOW_EXEC_COMMANDS is true */
	virtual bool Exec_Runtime( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) { return false; }

	/** Implementation of Exec that is only called in non-shipping targets */
	virtual bool Exec_Dev( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) { return false; }

	/** Implementation of Exec that is only called in editor */
	virtual bool Exec_Editor( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) { return false; }

};


