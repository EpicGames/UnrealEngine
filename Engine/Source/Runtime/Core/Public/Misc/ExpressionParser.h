// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Misc/ExpressionParserTypes.h"
#include "Templates/ValueOrError.h"

/** An expression parser, responsible for lexing, compiling, and evaluating expressions.
 *	The parser supports 3 functions:
 *		1. Lexing the expression into a set of user defined tokens,
 *		2. Compiling the tokenized expression to an efficient reverse-polish execution order,
 *		3. Evaluating the compiled tokens
 *
 *  See ExpressionParserExamples.cpp for example usage.
 */

namespace ExpressionParser
{
	template <typename CharType>
	using TLexResultType = TValueOrError<TArray<TExpressionToken<CharType>>, FExpressionError>;

	template <typename CharType>
	using TCompileResultType = TValueOrError<TArray<TCompiledToken<CharType>>, FExpressionError>;

	using LexResultType = TLexResultType<TCHAR>;
	using CompileResultType = TCompileResultType<TCHAR>;

	/** Lex the specified string, using the specified grammar */
	template <typename CharType>
	CORE_API TLexResultType<CharType> Lex(const CharType* InExpression, const TTokenDefinitions<CharType>& TokenDefinitions);

	/** Compile the specified expression into an array of Reverse-Polish order nodes for evaluation, according to our grammar definition */
	template <typename CharType>
	CORE_API TCompileResultType<CharType> Compile(const CharType* InExpression, const TTokenDefinitions<CharType>& TokenDefinitions, const FExpressionGrammar& InGrammar);

	/** Compile the specified tokens into an array of Reverse-Polish order nodes for evaluation, according to our grammar definition */
	template <typename CharType>
	CORE_API TCompileResultType<CharType> Compile(TArray<TExpressionToken<CharType>> InTokens, const FExpressionGrammar& InGrammar);

	/** Evaluate the specified expression using the specified token definitions, grammar definition, and evaluation environment */
	template <typename CharType>
	CORE_API FExpressionResult Evaluate(const CharType* InExpression, const TTokenDefinitions<CharType>& InTokenDefinitions, const FExpressionGrammar& InGrammar, const TIOperatorEvaluationEnvironment<CharType>& InEnvironment);

	/** Evaluate the specified pre-compiled tokens using an evaluation environment */
	template <typename CharType>
	CORE_API FExpressionResult Evaluate(const TArray<TCompiledToken<CharType>>& CompiledTokens, const TIOperatorEvaluationEnvironment<CharType>& InEnvironment);

	/** Templated versions of evaluation functions used when passing a specific jump table and context */
	template <typename CharType, typename ContextType>
	FExpressionResult Evaluate(const CharType* InExpression, const TTokenDefinitions<CharType>& InTokenDefinitions, const FExpressionGrammar& InGrammar,	const TOperatorJumpTable<ContextType>& InJumpTable, const ContextType* InContext = nullptr)
	{
		TOperatorEvaluationEnvironment<ContextType> Env(InJumpTable, InContext);
		return Evaluate(InExpression, InTokenDefinitions, InGrammar, Env);
	}

	template<typename ContextType>
	FExpressionResult Evaluate(const TArray<FCompiledToken>& CompiledTokens, const TOperatorJumpTable<ContextType>& InJumpTable, const ContextType* InContext = nullptr)
	{
		TOperatorEvaluationEnvironment<ContextType> Env(InJumpTable, InContext);
		return Evaluate(CompiledTokens, Env);
	}
}
