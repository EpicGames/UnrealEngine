// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ExpressionParserTypesFwd.h" // IWYU pragma: export
#include "Templates/UnrealTemplate.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/Function.h"
#include "Containers/Set.h"
#include "Containers/Map.h"
#include "Misc/Optional.h"
#include "Internationalization/Text.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Guid.h"
#include "Templates/ValueOrError.h"
#include <type_traits>

namespace Impl
{
	struct IExpressionNodeStorage;
}

/** Simple error structure used for reporting parse errors */
struct FExpressionError
{
	FExpressionError(const FText& InText)
		: Text(InText)
	{
	}

	FText Text;
};

/** Simple struct that defines a specific token contained in an FTokenStream  */
template <typename CharType>
class TStringToken
{
public:
	TStringToken() = default;

	/** Get the string representation of this token */
	TString<CharType> GetString() const
	{
		return TString<CharType>::ConstructFromPtrSize(TokenStart, (int32)(TokenEnd - TokenStart));
	}

	/** Check if this token is valid */
	bool IsValid() const
	{
		return TokenEnd != TokenStart;
	}

	/** Get the position of the start of this token in the stream */
	const CharType* GetTokenStartPos() const
	{
		return TokenStart;
	}

	/** Get the position of the end of this token in the stream */
	const CharType* GetTokenEndPos() const
	{
		return TokenEnd;
	}

	/** Get the character index of this token in the stream */
	int32 GetCharacterIndex() const
	{
		return CharacterIndex;
	}

	/** Get the line number of this token in the stream */
	int32 GetLineNumber() const
	{
		return LineNumber;
	}
	
	/** Accumulate another token into this one */
	void Accumulate(const TStringToken& InToken)
	{
		if (InToken.TokenEnd > TokenEnd)
		{
			TokenEnd = InToken.TokenEnd;
		}
	}

protected:
	friend class TTokenStream<CharType>;

	explicit TStringToken(const CharType* InStart, int32 Line = 0, int32 Character = 0)
		: TokenStart(InStart)
		, TokenEnd(InStart)
		, LineNumber(Line)
		, CharacterIndex(Character)
	{
	}

	const CharType* TokenStart = nullptr;
	const CharType* TokenEnd = nullptr;
	int32 LineNumber = 0;
	int32 CharacterIndex = 0;
};

/** Enum specifying how to treat the currently parsing character. */
enum class EParseState
{
	/** Include this character in the token and continue consuming */
	Continue,
	/** Include this character in the token and stop consuming */
	StopAfter,
	/** Exclude this character from the token and stop consuming */
	StopBefore, 
	/** Cancel parsing this token, and return nothing. */
	Cancel,
};

/** A token stream wraps up a raw string, providing accessors into it for consuming tokens */
template <typename CharType>
class TTokenStream
{
public:
	/**
	 * Parse out a token using the supplied predicate.
	 * Will keep consuming characters into the resulting token provided the predicate returns EParseState::Continue or EParseState::StopAfter.
	 * Optionally supply a token to accumulate into
	 * Returns a string token for the stream, or empty on error
	 */
	CORE_API TOptional<TStringToken<CharType>> ParseToken(TFunctionRef<EParseState(CharType)> Pred, TStringToken<CharType>* Accumulate = nullptr) const;

	/** Attempt parse out the specified pre-defined string from the current read position (or accumulating into the specified existing token) */
	CORE_API TOptional<TStringToken<CharType>> ParseToken(const CharType* Symbol, TStringToken<CharType>* Accumulate = nullptr) const;
	CORE_API TOptional<TStringToken<CharType>> ParseTokenIgnoreCase(const CharType* Symbol, TStringToken<CharType>* Accumulate = nullptr) const;

	/** Return a string token for the next character in the stream (or accumulating into the specified existing token) */
	CORE_API TOptional<TStringToken<CharType>> ParseSymbol(TStringToken<CharType>* Accumulate = nullptr) const;

	/** Attempt parse out the specified pre-defined string from the current read position (or accumulating into the specified existing token) */
	CORE_API TOptional<TStringToken<CharType>> ParseSymbol(CharType Symbol, TStringToken<CharType>* Accumulate = nullptr) const;

	/** Parse a whitespace token */
	CORE_API TOptional<TStringToken<CharType>> ParseWhitespace(TStringToken<CharType>* Accumulate = nullptr) const;

	/** Generate a token for the specified number of chars, at the current read position (or end of Accumulate) */
	CORE_API TOptional<TStringToken<CharType>> GenerateToken(int32 NumChars, TStringToken<CharType>* Accumulate = nullptr) const;

public:
	/** Constructor. The stream is only valid for the lifetime of the string provided */
	CORE_API explicit TTokenStream(const CharType* In);
	
	/** Peek at the character at the specified offset from the current read position */
	CORE_API CharType PeekChar(int32 Offset = 0) const;

	/** Get the number of characters remaining in the stream after the current read position */
	CORE_API int32 CharsRemaining() const;

	/** Check if it is valid to read (the optional number of characters) from the specified position */
	CORE_API bool IsReadPosValid(const CharType* InPos, int32 MinNumChars = 1) const;

	/** Check if the stream is empty */
	CORE_API bool IsEmpty() const;

	/** Get the current read position from the start of the stream */
	CORE_API int32 GetPosition() const;

	FORCEINLINE const CharType* GetStart() const
	{
		return Start;
	}
	FORCEINLINE const CharType* GetRead() const
	{
		return ReadPos;
	}
	FORCEINLINE const CharType* GetEnd() const
	{
		return End;
	}

	/** Get the error context from the current read position */
	CORE_API FString GetErrorContext() const;

	/** Set the current read position to the character proceeding the specified token */
	CORE_API void SetReadPos(const TStringToken<CharType>& Token);

private:
	/** The start of the expression */
	const CharType* Start;
	/** The end of the expression */
	const CharType* End;
	/** The current read position in the expression */
	const CharType* ReadPos;
};

/** Helper macro to define the necessary template specialization for a particular expression node type */
/** Variable length arguments are passed the FGuid constructor. Must be unique per type */
#define DEFINE_EXPRESSION_NODE_TYPE(TYPE, ...) \
template<> struct TGetExpressionNodeTypeId<TYPE>\
{\
	static const FGuid& GetTypeId()\
	{\
		static FGuid Global(__VA_ARGS__);\
		return Global;\
	}\
};
template<typename T> struct TGetExpressionNodeTypeId;

/** Primitive types should only be declared once inside the codebase to avoid conflicts */
DEFINE_EXPRESSION_NODE_TYPE(bool, 	0xCACBC715, 0x505A6B4A, 0x8808809F, 0x897AA5F6)
DEFINE_EXPRESSION_NODE_TYPE(double, 0x8444A8A3, 0x19AE4E13, 0xBCFA75EE, 0x39982BD6)

/**
 * A node in an expression.
 * 	Can be constructed from any C++ type that has a corresponding DEFINE_EXPRESSION_NODE_TYPE.
 * 	Evaluation behaviour (unary/binary operator etc) is defined in the expression grammar, rather than the type itself.
 */
class FExpressionNode
{
public:
	/** Default constructor */
	FExpressionNode() = default;

	/** Construction from client expression data type */
	template <
		typename T
		UE_REQUIRES(!std::is_convertible_v<T*, FExpressionNode*>)
	>
	FExpressionNode(T In);

	CORE_API ~FExpressionNode();

	// Movable, but non copyable
	CORE_API FExpressionNode(FExpressionNode&& In);
	FExpressionNode(const FExpressionNode&) = delete;
	CORE_API FExpressionNode& operator=(FExpressionNode&& In);
	FExpressionNode& operator=(const FExpressionNode&) = delete;

	/** Get the type identifier of this node */
	CORE_API const FGuid& GetTypeId() const;

	/** Cast this node to the specified type. Will return nullptr if the types do not match. */
	template<typename T>
	const T* Cast() const;

	/** Copy this node and its wrapped data */
	CORE_API FExpressionNode Copy() const;

private:
	/** The maximum size of type we will allow allocation on the stack (for efficiency). Anything larger will be allocated on the heap. */
	static constexpr uint32 MaxStackAllocationSize = 64 - sizeof(FGuid);

	/** Helper accessor to the data interface. Returns null for empty containers. */
	CORE_API Impl::IExpressionNodeStorage* GetData();
	CORE_API const Impl::IExpressionNodeStorage* GetData() const;

	/** TypeID - 16 bytes */
	FGuid TypeId;
	alignas(__STDCPP_DEFAULT_NEW_ALIGNMENT__) uint8 InlineBytes[MaxStackAllocationSize];
};

/** A specific token in a stream. Comprises an expression node, and the stream token it was created from */
template <typename CharType>
class TExpressionToken
{
public:
	explicit TExpressionToken(const TStringToken<CharType>& InContext, FExpressionNode InNode)
		: Node(MoveTemp(InNode))
		, Context(InContext)
	{
	}

	FExpressionNode Node;
	TStringToken<CharType> Context;
};

/** A compiled token, holding the token itself, and any compiler information required to evaluate it */
template <typename CharType>
struct TCompiledToken : TExpressionToken<CharType>
{
	// Todo: add callable types here?
	enum EType { Operand, PreUnaryOperator, PostUnaryOperator, BinaryOperator, ShortCircuit, Benign };

	explicit TCompiledToken(EType InType, TExpressionToken<CharType> InToken, TOptional<int32> InShortCircuitIndex = TOptional<int32>())
		: TExpressionToken<CharType>(MoveTemp(InToken))
		, Type(InType)
		, ShortCircuitIndex(InShortCircuitIndex)
	{
	}

	EType Type;
	TOptional<int32> ShortCircuitIndex;
};

/** Struct used to identify a function for a specific operator overload */
struct FOperatorFunctionID
{
	FGuid OperatorType;
	FGuid LeftOperandType;
	FGuid RightOperandType;

	friend bool operator==(const FOperatorFunctionID& A, const FOperatorFunctionID& B)
	{
		return A.OperatorType == B.OperatorType &&
			A.LeftOperandType == B.LeftOperandType &&
			A.RightOperandType == B.RightOperandType;
	}

	friend uint32 GetTypeHash(const FOperatorFunctionID& In)
	{
		const uint32 Hash = HashCombine(GetTypeHash(In.OperatorType), GetTypeHash(In.LeftOperandType));
		return HashCombine(GetTypeHash(In.RightOperandType), Hash);
	}
};

/** Jump table specifying how to execute an operator with different types */
template <typename ContextType, typename CharType>
struct TOperatorJumpTable
{
	/** Execute the specified token as a unary operator, if such an overload exists */
	FExpressionResult ExecPreUnary(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& R, const ContextType* Context) const;
	/** Execute the specified token as a unary operator, if such an overload exists */
	FExpressionResult ExecPostUnary(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& L, const ContextType* Context) const;
	/** Execute the specified token as a binary operator, if such an overload exists */
	FExpressionResult ExecBinary(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& L, const TExpressionToken<CharType>& R, const ContextType* Context) const;
	/** Check whether we should short circuit the specified operator */
	bool ShouldShortCircuit(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& L, const ContextType* Context) const;

	/**
	 * Map an expression node to a pre-unary operator with the specified implementation.
	 *
	 * The callable type must match the declaration Ret(Operand[, Context]), where:
	 *	 	Ret 	= Any DEFINE_EXPRESSION_NODE_TYPE type, OR FExpressionResult
	 *	 	Operand = Any DEFINE_EXPRESSION_NODE_TYPE type
	 *	 	Context = (optional) const ptr to user-supplied arbitrary context
	 *
	 * Examples that binds a '!' token to a function that attempts to do a boolean 'not':
	 *		JumpTable.MapPreUnary<FExclamation>([](bool A){ return !A; });
	 *		JumpTable.MapPreUnary<FExclamation>([](bool A, FMyContext* Ctxt){ if (Ctxt->IsBooleanNotOpEnabled()) { return !A; } else { return A; } });
	 *		JumpTable.MapPreUnary<FExclamation>([](bool A, const FMyContext* Ctxt) -> FExpressionResult {

	 *			if (Ctxt->IsBooleanNotOpEnabled())
	 *			{
	 *				return MakeValue(!A);
	 *			}
	 *			return MakeError(FExpressionError(LOCTEXT("NotNotEnabled", "Boolean not is not enabled.")));
	 *		});
	 */
	template<typename OperatorType, typename FuncType>
	void MapPreUnary(FuncType InFunc);

	/**
	 * Map an expression node to a post-unary operator with the specified implementation.
	 * The same function signature rules apply here as with MapPreUnary.
	 */
	template<typename OperatorType, typename FuncType>
	void MapPostUnary(FuncType InFunc);

	/**
	 * Map an expression node to a binary operator with the specified implementation.
	 *
	 * The callable type must match the declaration Ret(OperandL, OperandR, [, Context]), where:
	 *	 	Ret 		= Any DEFINE_EXPRESSION_NODE_TYPE type, OR FExpressionResult
	 *	 	OperandL	= Any DEFINE_EXPRESSION_NODE_TYPE type
	 *	 	OperandR	= Any DEFINE_EXPRESSION_NODE_TYPE type
	 *	 	Context 	= (optional) const ptr to user-supplied arbitrary context
	 *
	 * Examples that binds a '/' token to a function that attempts to do a division:
	 *		JumpTable.MapUnary<FForwardSlash>([](double A, double B){ return A / B; }); // Runtime exception on div/0 
	 *		JumpTable.MapUnary<FForwardSlash>([](double A, double B, FMyContext* Ctxt){
	 *			if (!Ctxt->IsMathEnabled())
	 *			{
	 *				return A;
	 *			}
	 *			return A / B; // Runtime exception on div/0 
	 *		});
	 *		JumpTable.MapUnary<FForwardSlash>([](double A, double B, const FMyContext* Ctxt) -> FExpressionResult {
	 *			if (!Ctxt->IsMathEnabled())
	 *			{
	 *				return MakeError(FExpressionError(LOCTEXT("MathNotEnabled", "Math is not enabled.")));
	 *			}
	 *			else if (B == 0)
	 *			{
	 *				return MakeError(FExpressionError(LOCTEXT("DivisionByZero", "Division by zero.")));	
	 *			}
	 *
	 *			return MakeValue(!A);
	 *		});
	 */
	template<typename OperatorType, typename FuncType>
	void MapBinary(FuncType InFunc);

	template<typename OperatorType, typename FuncType>
	void MapShortCircuit(FuncType InFunc);

public:
	using FUnaryFunction = TFunction<FExpressionResult(const FExpressionNode&, const ContextType* Context)>;
	using FBinaryFunction = TFunction<FExpressionResult(const FExpressionNode&, const FExpressionNode&, const ContextType* Context)>;
	using FShortCircuit = TFunction<bool(const FExpressionNode&, const ContextType* Context)>;

private:
	/** Maps of unary/binary operators */
	TMap<FOperatorFunctionID, FUnaryFunction>  PreUnaryOps;
	TMap<FOperatorFunctionID, FUnaryFunction>  PostUnaryOps;
	TMap<FOperatorFunctionID, FBinaryFunction> BinaryOps;
	TMap<FOperatorFunctionID, FShortCircuit>   BinaryShortCircuits;
};

/** Structures used for managing the evaluation environment for operators in an expression. This class manages the evaluation context
 * to avoid templating the whole evaluation code on a context type
 */
template <typename CharType>
struct TIOperatorEvaluationEnvironment
{
	/** Execute the specified token as a unary operator, if such an overload exists */
	virtual FExpressionResult ExecPreUnary(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& R) const = 0;
	/** Execute the specified token as a unary operator, if such an overload exists */
	virtual FExpressionResult ExecPostUnary(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& L) const = 0;
	/** Execute the specified token as a binary operator, if such an overload exists */
	virtual FExpressionResult ExecBinary(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& L, const TExpressionToken<CharType>& R) const = 0;
	/** Check whether we should short circuit the specified operator */
	virtual bool ShouldShortCircuit(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& L) const = 0;
};
template <typename ContextType, typename CharType>
struct TOperatorEvaluationEnvironment : TIOperatorEvaluationEnvironment<CharType>
{
	explicit TOperatorEvaluationEnvironment(const TOperatorJumpTable<ContextType, CharType>& InOperators, const ContextType* InContext)
		: Operators(InOperators)
		, Context(InContext)
	{
	}

	virtual FExpressionResult ExecPreUnary(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& R) const override
	{
		return Operators.ExecPreUnary(Operator, R, Context);
	}
	virtual FExpressionResult ExecPostUnary(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& L) const override
	{
		return Operators.ExecPostUnary(Operator, L, Context);
	}
	virtual FExpressionResult ExecBinary(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& L, const TExpressionToken<CharType>& R) const override
	{
		return Operators.ExecBinary(Operator, L, R, Context);
	}
	virtual bool ShouldShortCircuit(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& L) const override
	{
		return Operators.ShouldShortCircuit(Operator, L, Context);
	}

private:
	const TOperatorJumpTable<ContextType>& Operators;
	const ContextType* Context;
};

/** Class used to consume tokens from a string */
template <typename CharType>
class TExpressionTokenConsumer
{
public:
	/** Construction from a raw string. The consumer is only valid as long as the string is valid */
	CORE_API explicit TExpressionTokenConsumer(const CharType* InExpression);

	// Non-copyable
	TExpressionTokenConsumer(TExpressionTokenConsumer&&) = delete;
	TExpressionTokenConsumer(const TExpressionTokenConsumer&) = delete;
	TExpressionTokenConsumer& operator=(TExpressionTokenConsumer&&) = delete;
	TExpressionTokenConsumer& operator=(const TExpressionTokenConsumer&) = delete;
	~TExpressionTokenConsumer() = default;

	/** Extract the list of tokens from this consumer */
	CORE_API TArray<TExpressionToken<CharType>> Extract();

	/** Add an expression node to the consumer, specifying the FStringToken this node relates to.
	 *	Adding a node to the consumer will move its stream read position to the end of the added token.
	 */
	CORE_API void Add(const TStringToken<CharType>& SourceToken, FExpressionNode&& Node);

	/** Get the expression stream */
	CORE_API TTokenStream<CharType>& GetStream();

private:
	/** Array of added tokens */
	TArray<TExpressionToken<CharType>> Tokens;

	/** Stream that looks at the constructed expression */
	TTokenStream<CharType> Stream;
};

/** 
 * Typedefs that defines a function used to consume tokens
 * 	Definitions may add FExpressionNodes parsed from the provided consumer's stream, or return an optional error.
 *	Where a definition performs no mutable operations, subsequent token definitions will be invoked.
 */
template <typename CharType>
using TExpressionDefinition = TOptional<FExpressionError>(TExpressionTokenConsumer<CharType>&);


/** A lexeme dictionary defining how to lex an expression. */
template <typename CharType>
class TTokenDefinitions
{
public:
	TTokenDefinitions() = default;

	/** Define the grammar to ignore whitespace between tokens, unless explicitly included in a token */
	CORE_API void IgnoreWhitespace();

	/** Define a token by way of a function to be invoked to attempt to parse a token from a stream */
	CORE_API void DefineToken(TFunction<TExpressionDefinition<CharType>>&& Definition);

	/** Check if the grammar ignores whitespace */
	CORE_API bool DoesIgnoreWhitespace();

	/** Consume a token for the specified consumer */
	CORE_API TOptional<FExpressionError> ConsumeTokens(TExpressionTokenConsumer<CharType>& Consumer) const;

private:
	CORE_API TOptional<FExpressionError> ConsumeToken(TExpressionTokenConsumer<CharType>& Consumer) const;

private:
	bool bIgnoreWhitespace = false;
	TArray<TFunction<TExpressionDefinition<CharType>>> Definitions;
};


/**
 * Enum specifying the associativity (order of execution) for binary operators
 */
enum class EAssociativity : uint8
{
	RightToLeft,
	LeftToRight
};

/**
 * Struct for storing binary operator definition parameters
 */
struct FOpParameters
{
	/** The precedence of the operator */
	int32			Precedence;

	/** The associativity of the operator */
	EAssociativity	Associativity;

	/** Whether this operator can be short circuited or not */
	bool bCanShortCircuit;

	FOpParameters(int32 InPrecedence, EAssociativity InAssociativity, bool bInCanShortCircuit)
		: Precedence(InPrecedence)
		, Associativity(InAssociativity)
		, bCanShortCircuit(bInCanShortCircuit)
	{
	}
};

/** A lexical gammer defining how to parse an expression. Clients must define the tokens and operators to be interpreted by the parser. */
class FExpressionGrammar
{
public:
	/** Define a grouping operator from two expression node types */
	template<typename StartGroupType, typename EndGroupType>
	void DefineGrouping()
	{
		Groupings.Add(TGetExpressionNodeTypeId<StartGroupType>::GetTypeId(), TGetExpressionNodeTypeId<EndGroupType>::GetTypeId());
	}

	/** Define a pre-unary operator for the specified symbol */
	template<typename ExpressionNodeType>
	void DefinePreUnaryOperator()
	{
		PreUnaryOperators.Add(TGetExpressionNodeTypeId<ExpressionNodeType>::GetTypeId());
	}

	/** Define a post-unary operator for the specified symbol */
	template<typename ExpressionNodeType>
	void DefinePostUnaryOperator()
	{
		PostUnaryOperators.Add(TGetExpressionNodeTypeId<ExpressionNodeType>::GetTypeId());
	}

	/**
	 * Define a binary operator for the specified symbol, with the specified precedence and associativity
	 * NOTE: Associativity defaults to RightToLeft for legacy reasons.
	 *
	 * @param InPrecedence		The precedence (priority of execution) this operator should have
	 * @param InAssociativity	With operators of the same precedence, determines whether they execute left to right, or right to left
	 */
	template<typename ExpressionNodeType>
	void DefineBinaryOperator(int32 InPrecedence, EAssociativity InAssociativity=EAssociativity::RightToLeft, bool bCanShortCircuit = false)
	{
#if DO_CHECK
		for (TMap<FGuid, FOpParameters>::TConstIterator It(BinaryOperators); It; ++It)
		{
			const FOpParameters& CurValue = It.Value();

			if (CurValue.Precedence == InPrecedence)
			{
				// Operators of the same precedence, must all have the same associativity
				check(CurValue.Associativity == InAssociativity);
			}
		}
#endif

		BinaryOperators.Add(TGetExpressionNodeTypeId<ExpressionNodeType>::GetTypeId(), FOpParameters(InPrecedence, InAssociativity, bCanShortCircuit));
	}

public:

	/** Retrieve the corresponding grouping token for the specified open group type, or nullptr if it's not a group token */
	CORE_API const FGuid* GetGrouping(const FGuid& TypeId) const;

	/** Check if this grammar defines a pre-unary operator for the specified symbol */
	CORE_API bool HasPreUnaryOperator(const FGuid& TypeId) const;

	/** Check if this grammar defines a post-unary operator for the specified symbol */
	CORE_API bool HasPostUnaryOperator(const FGuid& TypeId) const;

	/** Get the binary operator precedence and associativity parameters, for the specified symbol, if any */
	CORE_API const FOpParameters* GetBinaryOperatorDefParameters(const FGuid& TypeId) const;

private:

	TMap<FGuid, FGuid>			Groupings;
	TSet<FGuid>					PreUnaryOperators;
	TSet<FGuid>					PostUnaryOperators;
	TMap<FGuid, FOpParameters>	BinaryOperators;
};

extern template class TTokenStream<ANSICHAR>;
extern template class TTokenStream<UTF8CHAR>;
extern template class TTokenStream<WIDECHAR>;

extern template class TExpressionTokenConsumer<ANSICHAR>;
extern template class TExpressionTokenConsumer<UTF8CHAR>;
extern template class TExpressionTokenConsumer<WIDECHAR>;

extern template class TTokenDefinitions<ANSICHAR>;
extern template class TTokenDefinitions<WIDECHAR>;
extern template class TTokenDefinitions<UTF8CHAR>;

#include "Misc/ExpressionParserTypes.inl" // IWYU pragma: export
