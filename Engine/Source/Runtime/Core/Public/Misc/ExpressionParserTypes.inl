// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include <type_traits>

#define LOCTEXT_NAMESPACE "ExpressionParser"

class FExpressionNode;
template <typename CharType> class TExpressionToken;
struct FOperatorFunctionID;

namespace Impl
{
	/** Interface for a wrapper utility for any moveable/copyable data */
	struct IExpressionNodeStorage
	{
		virtual ~IExpressionNodeStorage() = default;
		/** Move this type into another unallocated buffer (move-construct a new type from our wrapped value) */
		virtual void Reseat(uint8* Dst) = 0;
		/** Move this type to a buffer already allocated to the same type (uses type-defined move-assignment) */
		virtual void MoveAssign(uint8* Dst) = 0;
		/** Copy this data */
		virtual FExpressionNode Copy() const = 0;
	};

	/** Implementation of the wrapper utility for any moveable/copyable data, that allows us to virtually move/copy/destruct the data */
	/** Data is stored inline in this implementation, for efficiency */
 	template<typename T>
	struct FInlineDataStorage : IExpressionNodeStorage
	{
		/** The data itself, allocated on the stack */
		T Value;

		FInlineDataStorage(T InValue)
			: Value(MoveTemp(InValue))
		{
		}

		const T* Access() const
		{
			return &Value;
		}

		virtual void Reseat(uint8* Dst) override
		{
			::new((void*)Dst) FInlineDataStorage(MoveTemp(Value));
		}

		virtual void MoveAssign(uint8* Dst) override
		{
			reinterpret_cast<FInlineDataStorage*>(Dst)->Value = MoveTemp(Value);
		}

		virtual FExpressionNode Copy() const override
		{
			return Value;
		}
	};

	/** Data is stored on the heap in this implementation */
	template<typename T>
	struct FHeapDataStorage : IExpressionNodeStorage
	{
		/** The data itself, allocated on the heap */
		TUniquePtr<T> Value;
		
		/** Constructor/destructor */
		FHeapDataStorage(T InValue)
			: Value(MakeUnique<T>(MoveTemp(InValue)))
		{
		}

		const T* Access() const
		{
			return Value.Get();
		}

		virtual void Reseat(uint8* Dst) override
		{
			::new((void*)Dst) FHeapDataStorage(MoveTemp(*this));
		}

		virtual void MoveAssign(uint8* Dst) override
		{
			reinterpret_cast<FHeapDataStorage*>(Dst)->Value = MoveTemp(Value);
		}
		virtual FExpressionNode Copy() const override
		{
			return *Value;
		}
	};

	template<typename T, uint32 MaxStackAllocationSize>
	struct TStorageTypeDeduction
	{
		using Type = std::conditional_t<sizeof(FInlineDataStorage<T>) <= MaxStackAllocationSize, FInlineDataStorage<T>, FHeapDataStorage<T>>;
	};


	/** Machinery required for operator mapping */
	template <typename> struct TCallableInfo;
	template <typename> struct TCallableInfoImpl;

	template <typename Ret_, typename T, typename Arg1_>
	struct TCallableInfoImpl<Ret_ (T::*)(Arg1_)>
	{
		using Ret = Ret_;
		using Arg1 = Arg1_;
		static constexpr uint32 NumArgs = 1;
	};

	template <typename Ret_, typename T, typename Arg1_>
	struct TCallableInfoImpl<Ret_ (T::*)(Arg1_) const>
	{
		using Ret = Ret_;
		using Arg1 = Arg1_;
		static constexpr uint32 NumArgs = 1;
	};

	template <typename Ret_, typename T, typename Arg1_, typename Arg2_>
	struct TCallableInfoImpl<Ret_ (T::*)(Arg1_, Arg2_)>
	{
		using Ret = Ret_;
		using Arg1 = Arg1_;
		using Arg2 = Arg2_;
		static constexpr uint32 NumArgs = 2;
	};

	template <typename Ret_, typename T, typename Arg1_, typename Arg2_>
	struct TCallableInfoImpl<Ret_ (T::*)(Arg1_, Arg2_) const>
	{
		using Ret = Ret_;
		using Arg1 = Arg1_;
		using Arg2 = Arg2_;
		static constexpr uint32 NumArgs = 2;
	};
	
	template <typename Ret_, typename T, typename Arg1_, typename Arg2_, typename Arg3_>
	struct TCallableInfoImpl<Ret_ (T::*)(Arg1_, Arg2_, Arg3_)>
	{
		using Ret = Ret_;
		using Arg1 = Arg1_;
		using Arg2 = Arg2_;
		using Arg3 = Arg3_;
		static constexpr uint32 NumArgs = 3;
	};

	template <typename Ret_, typename T, typename Arg1_, typename Arg2_, typename Arg3_>
	struct TCallableInfoImpl<Ret_ (T::*)(Arg1_, Arg2_, Arg3_) const>
	{
		using Ret = Ret_;
		using Arg1 = Arg1_;
		using Arg2 = Arg2_;
		using Arg3 = Arg3_;
		static constexpr uint32 NumArgs = 3;
	};

	template <typename T>
	struct TCallableInfo : TCallableInfoImpl<decltype(&T::operator())>
	{
	};

	/** Overloaded function that returns an FExpressionResult, regardless of what is passed in */
	template<typename T>
	inline FExpressionResult ForwardReturnType(T&& Result)
	{
		return MakeValue(MoveTemp(Result));
	}
	inline FExpressionResult ForwardReturnType(FExpressionResult&& Result)
	{
		return MoveTemp(Result);
	}

	/** Wrapper function for supplied functions of the signature T(A) */
	template<typename OperandType, typename ContextType, typename FuncType>
	inline typename TOperatorJumpTable<ContextType>::FUnaryFunction WrapUnaryFunction(FuncType In)
	{
		constexpr uint32 NumArgs = Impl::TCallableInfo<FuncType>::NumArgs;

		if constexpr (NumArgs == 1)
		{
			// Ignore the context
			return [=](const FExpressionNode& InOperand, const ContextType* Context)
			{
				return ForwardReturnType(In(*InOperand.Cast<OperandType>()));
			};
		}
		else if constexpr (NumArgs == 2)
		{
			return [=](const FExpressionNode& InOperand, const ContextType* Context)
			{
				return ForwardReturnType(In(*InOperand.Cast<OperandType>(), Context));
			};
		}
		else
		{
			// sizeof(OperandType) == 0 is used to create a false value based on a template parameter
			static_assert(sizeof(OperandType) == 0, "FuncType has an unexpected number of parameters");
			return nullptr;
		}
	}

	/** Wrapper function for supplied functions of the signature T(A, B) */
	template<typename LeftOperandType, typename RightOperandType, typename ContextType, typename FuncType>
	inline typename TOperatorJumpTable<ContextType>::FBinaryFunction WrapBinaryFunction(FuncType In)
	{
		constexpr uint32 NumArgs = Impl::TCallableInfo<FuncType>::NumArgs;

		if constexpr (NumArgs == 2)
		{
			// Ignore the context
			return [=](const FExpressionNode& InLeftOperand, const FExpressionNode& InRightOperand, const ContextType* Context)
			{
				return ForwardReturnType(In(*InLeftOperand.Cast<LeftOperandType>(), *InRightOperand.Cast<RightOperandType>()));
			};
		}
		else if constexpr (NumArgs == 3)
		{
			return [=](const FExpressionNode& InLeftOperand, const FExpressionNode& InRightOperand, const ContextType* Context)
			{
				return ForwardReturnType(In(*InLeftOperand.Cast<LeftOperandType>(), *InRightOperand.Cast<RightOperandType>(), Context));
			};
		}
		else
		{
			// sizeof(LeftOperandType) == 0 is used to create a false value based on a template parameter
			static_assert(sizeof(LeftOperandType) == 0, "FuncType has an unexpected number of parameters");
			return nullptr;
		}
	}

	/** Wrapper function for supplied functions of the signature bool(A, const ContextType* Context) */
	template<typename OperandType, typename ContextType, typename FuncType>
	inline typename TOperatorJumpTable<ContextType>::FShortCircuit WrapShortCircuitFunction(FuncType In)
	{
		constexpr uint32 NumArgs = Impl::TCallableInfo<FuncType>::NumArgs;

		if constexpr (NumArgs == 1)
		{
			// Ignore the context
			return [=](const FExpressionNode& InOperand, const ContextType* Context)
			{
				return In(*InOperand.Cast<OperandType>());
			};
		}
		else if constexpr (NumArgs == 2)
		{
			return [=](const FExpressionNode& InOperand, const ContextType* Context)
			{
				return In(*InOperand.Cast<OperandType>(), Context);
			};
		}
		else
		{
			// sizeof(OperandType) == 0 is used to create a false value based on a template parameter
			static_assert(sizeof(OperandType) == 0, "FuncType has an unexpected number of parameters");
			return nullptr;
		}
	}
}

template <
	typename T
	UE_REQUIRES_DEFINITION(!std::is_convertible_v<T*, FExpressionNode*>)
>
FExpressionNode::FExpressionNode(T In)
	: TypeId(TGetExpressionNodeTypeId<T>::GetTypeId())
{
	// Choose the relevant allocation strategy based on the size of the type
	::new((void*)InlineBytes) typename Impl::TStorageTypeDeduction<T, MaxStackAllocationSize>::Type(MoveTemp(In));
}

template<typename T>
const T* FExpressionNode::Cast() const
{
	if (TypeId == TGetExpressionNodeTypeId<T>::GetTypeId())
	{
		return reinterpret_cast<const typename Impl::TStorageTypeDeduction<T, MaxStackAllocationSize>::Type*>(InlineBytes)->Access();
	}
	return nullptr;
}

// End FExpresionNode

template<typename ContextType, typename CharType>
FExpressionResult TOperatorJumpTable<ContextType, CharType>::ExecBinary(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& L, const TExpressionToken<CharType>& R, const ContextType* Context) const
{
	FOperatorFunctionID ID = { Operator.Node.GetTypeId(), L.Node.GetTypeId(), R.Node.GetTypeId() };
	if (const auto* Func = BinaryOps.Find(ID))
	{
		return (*Func)(L.Node, R.Node, Context);
	}

	FFormatOrderedArguments Args;
	Args.Add(FText::FromString(Operator.Context.GetString()));
	Args.Add(FText::FromString(L.Context.GetString()));
	Args.Add(FText::FromString(R.Context.GetString()));
	return MakeError(FText::Format(LOCTEXT("BinaryExecutionError", "Binary operator {0} cannot operate on {1} and {2}"), Args));
}

template<typename ContextType, typename CharType>
bool TOperatorJumpTable<ContextType, CharType>::ShouldShortCircuit(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& L, const ContextType* Context) const
{
	FOperatorFunctionID ID = { Operator.Node.GetTypeId(), L.Node.GetTypeId(), FGuid() };
	if (const auto* Func = BinaryShortCircuits.Find(ID))
	{
		return (*Func)(L.Node, Context);
	}

	return false;
}

template<typename ContextType, typename CharType>
FExpressionResult TOperatorJumpTable<ContextType, CharType>::ExecPreUnary(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& R, const ContextType* Context) const
{
	FOperatorFunctionID ID = { Operator.Node.GetTypeId(), FGuid(), R.Node.GetTypeId() };
	if (const auto* Func = PreUnaryOps.Find(ID))
	{
		return (*Func)(R.Node, Context);
	}

	FFormatOrderedArguments Args;
	Args.Add(FText::FromString(Operator.Context.GetString()));
	Args.Add(FText::FromString(R.Context.GetString()));
	return MakeError(FText::Format(LOCTEXT("PreUnaryExecutionError", "Pre-unary operator {0} cannot operate on {1}"), Args));
}

template<typename ContextType, typename CharType>
FExpressionResult TOperatorJumpTable<ContextType, CharType>::ExecPostUnary(const TExpressionToken<CharType>& Operator, const TExpressionToken<CharType>& L, const ContextType* Context) const
{
	FOperatorFunctionID ID = { Operator.Node.GetTypeId(), L.Node.GetTypeId(), FGuid() };
	if (const auto* Func = PostUnaryOps.Find(ID))
	{
		return (*Func)(L.Node, Context);
	}

	FFormatOrderedArguments Args;
	Args.Add(FText::FromString(Operator.Context.GetString()));
	Args.Add(FText::FromString(L.Context.GetString()));
	return MakeError(FText::Format(LOCTEXT("PostUnaryExecutionError", "Post-unary operator {0} cannot operate on {1}"), Args));
}

template<typename ContextType, typename CharType>
template<typename OperatorType, typename FuncType>
void TOperatorJumpTable<ContextType, CharType>::MapPreUnary(FuncType InFunc)
{
	using OperandType = std::decay_t<typename Impl::TCallableInfo<FuncType>::Arg1>;

	FOperatorFunctionID ID = {
		TGetExpressionNodeTypeId<OperatorType>::GetTypeId(),
		FGuid(),
		TGetExpressionNodeTypeId<OperandType>::GetTypeId()
	};

	PreUnaryOps.Add(ID, Impl::WrapUnaryFunction<OperandType, ContextType>(InFunc));
}

template<typename ContextType, typename CharType>
template<typename OperatorType, typename FuncType>
void TOperatorJumpTable<ContextType, CharType>::MapPostUnary(FuncType InFunc)
{
	using OperandType = std::decay_t<typename Impl::TCallableInfo<FuncType>::Arg1>;

	FOperatorFunctionID ID = {
		TGetExpressionNodeTypeId<OperatorType>::GetTypeId(),
		TGetExpressionNodeTypeId<OperandType>::GetTypeId(),
		FGuid()
	};

	PostUnaryOps.Add(ID, Impl::WrapUnaryFunction<OperandType, ContextType>(InFunc));
}

template<typename ContextType, typename CharType>
template<typename OperatorType, typename FuncType>
void TOperatorJumpTable<ContextType, CharType>::MapBinary(FuncType InFunc)
{
	using LeftOperandType = std::decay_t<typename Impl::TCallableInfo<FuncType>::Arg1>;
	using RightOperandType = std::decay_t<typename Impl::TCallableInfo<FuncType>::Arg2>;

	FOperatorFunctionID ID = {
		TGetExpressionNodeTypeId<OperatorType>::GetTypeId(),
		TGetExpressionNodeTypeId<LeftOperandType>::GetTypeId(),
		TGetExpressionNodeTypeId<RightOperandType>::GetTypeId()
	};

	BinaryOps.Add(ID, Impl::WrapBinaryFunction<LeftOperandType, RightOperandType, ContextType>(InFunc));
}

template<typename ContextType, typename CharType>
template<typename OperatorType, typename FuncType>
void TOperatorJumpTable<ContextType, CharType>::MapShortCircuit(FuncType InFunc)
{
	using OperandType = std::decay_t<typename Impl::TCallableInfo<FuncType>::Arg1>;

	FOperatorFunctionID ID = {
		TGetExpressionNodeTypeId<OperatorType>::GetTypeId(),
		TGetExpressionNodeTypeId<OperandType>::GetTypeId(),
		FGuid()
	};

	BinaryShortCircuits.Add(ID, Impl::WrapShortCircuitFunction<OperandType, ContextType>(InFunc));
}


using FOperatorEvaluationEnvironment = TOperatorEvaluationEnvironment<>;

#undef LOCTEXT_NAMESPACE
