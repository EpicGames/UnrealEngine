// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/OptionalFwd.h"
#include "Templates/ValueOrError.h"

struct FExpressionError;
struct FOperatorFunctionID;
enum class EAssociativity : uint8;
struct FOpParameters;
class FExpressionGrammar;
class FExpressionNode;

template <typename CharType>
class TStringToken;
template <typename CharType>
class TTokenStream;
template <typename CharType>
class TExpressionToken;
template <typename CharType>
struct TCompiledToken;
template <typename ContextType=void, typename CharType = TCHAR>
struct TOperatorJumpTable;
template <typename CharType>
struct TIOperatorEvaluationEnvironment;
template <typename ContextType = void, typename CharType = TCHAR>
struct TOperatorEvaluationEnvironment;
template <typename CharType>
class TExpressionTokenConsumer;
template <typename CharType>
class TTokenDefinitions;

using FExpressionResult = TValueOrError<FExpressionNode, FExpressionError>;
using FOperatorJumpTable = TOperatorJumpTable<>;
using FOperatorEvaluationEnvironment = TOperatorEvaluationEnvironment<>;
using FStringToken = TStringToken<TCHAR>;
using FTokenStream = TTokenStream<TCHAR>;
using FExpressionToken = TExpressionToken<TCHAR>;
using FCompiledToken = TCompiledToken<TCHAR>;
using FExpressionTokenConsumer = TExpressionTokenConsumer<TCHAR>;
using FTokenDefinitions = TTokenDefinitions<TCHAR>;
