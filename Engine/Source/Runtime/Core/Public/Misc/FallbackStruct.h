// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// A struct used as stub for deleted ones.
struct FFallbackStruct
{
};