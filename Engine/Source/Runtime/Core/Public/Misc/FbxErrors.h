// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/TokenizedMessage.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"


/**
 * This file contains known map errors that can be referenced by name.
 * Documentation for these errors is assumed to lie in UDN at: Engine\Documentation\Source\Shared\Editor\MapErrors
 */
struct FFbxErrors
{
	/** Generic */
	static CORE_API FLazyName Generic_ImportingNewObjectFailed;

	static CORE_API FLazyName Generic_ReimportingObjectFailed;

	static CORE_API FLazyName Generic_LoadingSceneFailed;

	static CORE_API FLazyName Generic_InvalidCharacterInName;

	static CORE_API FLazyName Generic_SameNameAssetExists;

	static CORE_API FLazyName Generic_SameNameAssetOverriding;

	static CORE_API FLazyName Generic_CannotDeleteReferenced;

	static CORE_API FLazyName Generic_FBXFileParseFailed;

	static CORE_API FLazyName Generic_MeshNotFound;

	static CORE_API FLazyName Generic_CannotDetectImportType;

	/** Mesh Generic **/

	static CORE_API FLazyName Generic_Mesh_NoGeometry;

	static CORE_API FLazyName Generic_Mesh_SmallGeometry;

	static CORE_API FLazyName Generic_Mesh_TriangulationFailed;

	static CORE_API FLazyName Generic_Mesh_ConvertSmoothingGroupFailed;

	static CORE_API FLazyName Generic_Mesh_UnsupportingSmoothingGroup;

	static CORE_API FLazyName Generic_Mesh_MaterialIndexInconsistency;

	static CORE_API FLazyName Generic_Mesh_MeshNotFound;

	static CORE_API FLazyName Generic_Mesh_NoSmoothingGroup;

	static CORE_API FLazyName Generic_Mesh_LOD_InvalidIndex;

	static CORE_API FLazyName Generic_Mesh_LOD_NoFileSelected;

	static CORE_API FLazyName Generic_Mesh_LOD_MultipleFilesSelected;

	static CORE_API FLazyName Generic_Mesh_SkinxxNameError;

	static CORE_API FLazyName Generic_Mesh_TooManyLODs;

	static CORE_API FLazyName Generic_Mesh_TangentsComputeError;

	static CORE_API FLazyName Generic_Mesh_NoReductionModuleAvailable;

	static CORE_API FLazyName Generic_Mesh_TooMuchUVChannels;

	/** Static Mesh **/
	static CORE_API FLazyName StaticMesh_TooManyMaterials;

	static CORE_API FLazyName StaticMesh_UVSetLayoutProblem;

	static CORE_API FLazyName StaticMesh_NoTriangles;

	static CORE_API FLazyName StaticMesh_BuildError;

	static CORE_API FLazyName StaticMesh_AllTrianglesDegenerate;

	static CORE_API FLazyName StaticMesh_AdjacencyOptionForced;

	/** SkeletalMesh **/
	static CORE_API FLazyName SkeletalMesh_DifferentRoots;

	static CORE_API FLazyName SkeletalMesh_DuplicateBones;

	static CORE_API FLazyName SkeletalMesh_NoInfluences;

	static CORE_API FLazyName SkeletalMesh_TooManyInfluences;

	static CORE_API FLazyName SkeletalMesh_RestoreSortingMismatchedStrips;

	static CORE_API FLazyName SkeletalMesh_RestoreSortingNoSectionMatch;

	static CORE_API FLazyName SkeletalMesh_RestoreSortingForSectionNumber;

	static CORE_API FLazyName SkeletalMesh_NoMeshFoundOnRoot;
	
	static CORE_API FLazyName SkeletalMesh_InvalidRoot;

	static CORE_API FLazyName SkeletalMesh_InvalidBone;

	static CORE_API FLazyName SkeletalMesh_InvalidNode;

	static CORE_API FLazyName SkeletalMesh_NoWeightsOnDeformer;

	static CORE_API FLazyName SkeletalMesh_NoBindPoseInScene;

	static CORE_API FLazyName SkeletalMesh_NoAssociatedCluster;

	static CORE_API FLazyName SkeletalMesh_NoBoneFound;

	static CORE_API FLazyName SkeletalMesh_InvalidBindPose;

	static CORE_API FLazyName SkeletalMesh_MultipleRoots;

	static CORE_API FLazyName SkeletalMesh_BonesAreMissingFromBindPose;

	static CORE_API FLazyName SkeletalMesh_VertMissingInfluences;

	static CORE_API FLazyName SkeletalMesh_SectionWithNoTriangle;

	static CORE_API FLazyName SkeletalMesh_TooManyVertices;

	static CORE_API FLazyName SkeletalMesh_FailedToCreatePhyscisAsset;

	static CORE_API FLazyName SkeletalMesh_SkeletonRecreateError;

	static CORE_API FLazyName SkeletalMesh_ExceedsMaxBoneCount;

	static CORE_API FLazyName SkeletalMesh_NoUVSet;

	static CORE_API FLazyName SkeletalMesh_LOD_MissingBone;

	static CORE_API FLazyName SkeletalMesh_LOD_FailedToImport;

	static CORE_API FLazyName SkeletalMesh_LOD_RootNameIncorrect;

	static CORE_API FLazyName SkeletalMesh_LOD_BonesDoNotMatch;

	static CORE_API FLazyName SkeletalMesh_LOD_IncorrectParent;

	static CORE_API FLazyName SkeletalMesh_LOD_HasSoftVerts;

	static CORE_API FLazyName SkeletalMesh_LOD_MissingSocketBone;

	static CORE_API FLazyName SkeletalMesh_LOD_MissingMorphTarget;

	static CORE_API FLazyName SkeletalMesh_FillImportDataFailed;

	static CORE_API FLazyName SkeletalMesh_InvalidPosition;

	static CORE_API FLazyName SkeletalMesh_AttributeComponentCountMismatch;

	/** Animation **/
	static CORE_API FLazyName Animation_CouldNotFindRootTrack;

	static CORE_API FLazyName Animation_CouldNotBuildSkeleton;

	static CORE_API FLazyName Animation_CouldNotFindTrack;

	static CORE_API FLazyName Animation_ZeroLength;

	static CORE_API FLazyName Animation_RootTrackMismatch;

	static CORE_API FLazyName Animation_DuplicatedBone;

	static CORE_API FLazyName Animation_MissingBones;

	static CORE_API FLazyName Animation_InvalidData;

	static CORE_API FLazyName Animation_TransformError;

	static CORE_API FLazyName Animation_DifferentLength;

	static CORE_API FLazyName Animation_CurveNotFound;
};

/**
 * Map error specific message token.
 */
class FFbxErrorToken : public FDocumentationToken
{
public:
	/** Factory method, tokens can only be constructed as shared refs */
	static TSharedRef<FFbxErrorToken> Create( const FName& InErrorName )
	{
		return MakeShareable(new FFbxErrorToken(InErrorName));
	}

private:
	/** Private constructor */
	CORE_API FFbxErrorToken( const FName& InErrorName );
};
