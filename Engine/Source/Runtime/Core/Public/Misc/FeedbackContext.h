// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreDelegates.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformTime.h"
#include "Internationalization/Text.h"
#include "Misc/OutputDevice.h"
#include "Misc/ScopeLock.h"
#include "Misc/SlowTask.h"
#include "Misc/SlowTaskStack.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

class FContextSupplier;
class SBuildProgressWidget;
struct FSlowTask;

/** A context for displaying modal warning messages. */
class FFeedbackContext
	: public FOutputDevice
{
public:
	CORE_API virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override;
	CORE_API virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time) override;
	CORE_API virtual void SerializeRecord(const UE::FLogRecord& Record) override;

	/** Ask the user a binary question, returning their answer */
	CORE_API virtual bool YesNof( const FText& Question );
	
	/**
	 * Whether or not the user has canceled out of the progress dialog
	 * (i.e. the ongoing slow task or the last one that ran).
	 * The user cancel flag is reset when starting a new root slow task.
	 */
	virtual bool ReceivedUserCancel() { return false; }
	
	/** Public const access to the current state of the scope stack */
	UE_FORCEINLINE_HINT const FSlowTaskStack& GetScopeStack() const
	{
		return ScopeStack;
	}

	DECLARE_EVENT_OneParam(FFeedbackContext, FOnStartSlowTask, const FText& TaskName );
	FOnStartSlowTask& OnStartSlowTask() { return StartSlowTaskEvent; }

	DECLARE_EVENT_TwoParams(FFeedbackContext, FOnFinalizeSlowTask, const FText& TaskName, double DurationInSeconds);
	FOnFinalizeSlowTask& OnFinalizeSlowTask() { return FinalizeSlowTaskEvent; }

	DECLARE_EVENT_TwoParams(FFeedbackContext, FOnStartSlowTaskWithGuid, FGuid Guid, const FText& TaskName);
	FOnStartSlowTaskWithGuid& OnStartSlowTaskWithGuid() { return StartSlowTaskWithGuidEvent; }

	DECLARE_EVENT_TwoParams(FFeedbackContext, FOnFinalizeSlowTaskWithGuid, FGuid Guid, const FText& TaskName);
	FOnFinalizeSlowTaskWithGuid& OnFinalizeSlowTaskWithGuid() { return FinalizeSlowTaskWithGuidEvent; }

	/**** Legacy API - not deprecated as it's still in heavy use, but superceded by FScopedSlowTask ****/
	CORE_API void BeginSlowTask( const FText& Task, bool ShowProgressDialog, bool bShowCancelButton=false );
	CORE_API void UpdateProgress( int32 Numerator, int32 Denominator );
	CORE_API void StatusUpdate( int32 Numerator, int32 Denominator, const FText& StatusText );
	CORE_API void StatusForceUpdate( int32 Numerator, int32 Denominator, const FText& StatusText );
	CORE_API void EndSlowTask();
	/**** end legacy API ****/

protected:

	/**
	 * Called to create a slow task
	 */
	virtual void StartSlowTask( const FText& Task, bool bShowCancelButton=false )
	{
		TaskName = Task;
		TaskGuid = FGuid::NewGuid();
		TaskStartTime = FPlatformTime::Seconds();
		GIsSlowTask = true;
		StartSlowTaskEvent.Broadcast(TaskName);
		StartSlowTaskWithGuidEvent.Broadcast(TaskGuid, TaskName);
	}

	/**
	 * Called to destroy a slow task
	 */
	virtual void FinalizeSlowTask( )
	{
		const double TaskDuration = FPlatformTime::Seconds() - TaskStartTime;
		FinalizeSlowTaskEvent.Broadcast(TaskName, TaskDuration);
		FinalizeSlowTaskWithGuidEvent.Broadcast(TaskGuid, TaskName);
		GIsSlowTask = false;
	}

	/**
	 * Called when some progress has occurred
	 * @param	TotalProgressInterp		[0..1] Value indicating the total progress of the slow task
	 * @param	DisplayMessage			The message to display on the slow task
	 */
	virtual void ProgressReported( const float TotalProgressInterp, FText DisplayMessage )
	{
		if (TotalProgressInterp > 0)
		{
			FCoreDelegates::ApplicationHeartbeat.Broadcast();
		}
	}

	/** Called to check whether we are playing in editor when starting a slow task */
	CORE_API virtual bool IsPlayingInEditor() const;

	CORE_API ELogVerbosity::Type ResolveVerbosity(ELogVerbosity::Type Verbosity) const;

	CORE_API void FormatLine(FStringBuilderBase& Out, const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time, ELogVerbosity::Type* OutVerbosity = nullptr) const;
	CORE_API void FormatRecordLine(FStringBuilderBase& Out, const UE::FLogRecord& Record, ELogVerbosity::Type* OutVerbosity = nullptr) const;

public:
	virtual FContextSupplier* GetContext() const { return nullptr; }
	virtual void SetContext(FContextSupplier* InContext) {}

	/** Shows/Closes Special Build Progress dialogs */
	virtual TWeakPtr<class SBuildProgressWidget> ShowBuildProgressWindow() {return TWeakPtr<class SBuildProgressWidget>();}
	virtual void CloseBuildProgressWindow() {}

	/** Promote any logged warnings so that they act as errors */
	bool TreatWarningsAsErrors = false;
	/** Demote any logged errors so that they act as warnings; takes priority over TreatWarningsAsErrors */
	bool TreatErrorsAsWarnings = false;

	CORE_API FFeedbackContext();
	CORE_API virtual ~FFeedbackContext();

	/** Gets warnings history */
	void GetWarnings(TArray<FString>& OutWarnings) const
	{
		FScopeLock WarningsAndErrorsLock(&WarningsAndErrorsCritical);
		OutWarnings = Warnings;
	}
	int32 GetNumWarnings() const
	{
		return Warnings.Num();
	}

	/** Gets errors history */
	void GetErrors(TArray<FString>& OutErrors) const
	{
		FScopeLock WarningsAndErrorsLock(&WarningsAndErrorsCritical);
		OutErrors = Errors;
	}
	int32 GetNumErrors() const
	{
		return Errors.Num();
	}

	/** Gets all errors and warnings and clears the history */
	void GetErrorsAndWarningsAndEmpty(TArray<FString>& OutWarningsAndErrors)
	{
		FScopeLock WarningsAndErrorsLock(&WarningsAndErrorsCritical);
		OutWarningsAndErrors = MoveTemp(Errors);
		OutWarningsAndErrors += MoveTemp(Warnings);
	}
	/** Clears all history */
	void ClearWarningsAndErrors()
	{
		FScopeLock WarningsAndErrorsLock(&WarningsAndErrorsCritical);
		Errors.Empty();
		Warnings.Empty();
	}

private:
	CORE_API void AddToHistory(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time);
	CORE_API void AddRecordToHistory(const UE::FLogRecord& Record);

	CORE_API FFeedbackContext(const FFeedbackContext&);
	CORE_API FFeedbackContext& operator=(const FFeedbackContext&);

	/** Warnings history */
	TArray<FString> Warnings;
	/** Errors history */
	TArray<FString> Errors;
	/** Guard for the errors and warnings history */
	mutable FCriticalSection WarningsAndErrorsCritical;

	/** The name of any task we are running */
	FText TaskName;

	double TaskStartTime;
	FOnStartSlowTask StartSlowTaskEvent;
	FOnFinalizeSlowTask FinalizeSlowTaskEvent;
	
	FGuid TaskGuid;
	FOnStartSlowTaskWithGuid StartSlowTaskWithGuidEvent;
	FOnFinalizeSlowTaskWithGuid FinalizeSlowTaskWithGuidEvent;

protected:
	
	friend FSlowTask;

	/** Stack of pointers to feedback scopes that are currently open */
	FSlowTaskStack ScopeStack;

	/**
	 * Points to the ScopeStack above when initialized - this is because Slate wants a TSharedPtr,
	 * but we don't want to allocate
	 */
	mutable TSharedPtr<FSlowTaskStack> ScopeStackSharedPtr;

	const TSharedPtr<FSlowTaskStack>& GetScopeStackSharedPtr() const
	{
		if (!ScopeStackSharedPtr)
		{
			ScopeStackSharedPtr = MakeShareable(const_cast<FSlowTaskStack*>(&ScopeStack), [](FSlowTaskStack*){});
		}
		return ScopeStackSharedPtr;
	}

	TArray<TUniquePtr<FSlowTask>> LegacyAPIScopes;

	/** Ask that the UI be updated as a result of the scope stack changing */
	CORE_API void RequestUpdateUI(bool bForceUpdate = false);

	/** Update the UI as a result of the scope stack changing */
	CORE_API void UpdateUI();

	/**
	 * Adds a new warning message to warnings history.
	 * @param InWarning Warning message
	 */
	void AddWarning(const FString& InWarning)
	{
		FScopeLock WarningsAndErrorsLock(&WarningsAndErrorsCritical);
		Warnings.Add(InWarning);
	}
	void AddWarning(FString&& InWarning)
	{
		FScopeLock WarningsAndErrorsLock(&WarningsAndErrorsCritical);
		Warnings.Add(MoveTemp(InWarning));
	}

	/**
	* Adds a new error message to errors history.
	* @param InWarning Error message
	*/
	void AddError(const FString& InError)
	{
		FScopeLock WarningsAndErrorsLock(&WarningsAndErrorsCritical);
		Errors.Add(InError);
	}
	void AddError(FString&& InError)
	{
		FScopeLock WarningsAndErrorsLock(&WarningsAndErrorsCritical);
		Errors.Add(MoveTemp(InError));
	}
};
