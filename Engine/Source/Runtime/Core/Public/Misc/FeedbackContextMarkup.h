// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreFwd.h"
#include "CoreTypes.h"

class FFeedbackContext;
class FText;

/**
 * A text-based markup language can be used to allow external processes to control the state 
 * machine, allowing command-line utilities to give graphical user feedback.
 *
 * Example markup:
 *
 * Update the progress of the current operation:
 *		@progress 10/20
 *		@progress 50%
 *
 * Update the progress and set a status message for the current operation:
 *		@progress 'Compiling source code...' 50%
 */
class FFeedbackContextMarkup
{
public:

	/** Markup stack manipulation. */
	static CORE_API bool ParseCommand(const FString& Line, FFeedbackContext* Warn);

	/** Utility functions for dealing with external processes. */
	static CORE_API bool PipeProcessOutput(const FText& Description, const FString& URL, const FString& Params, FFeedbackContext* Warn, int32* OutExitCode);

private:

	/** Try to read a single exact-match token from the input stream. Must be followed by whitespace or EOL. */
	static bool ReadToken(const TCHAR *&Text, const TCHAR *Token);

	/** Read a progress value from the input stream. Valid forms are <Numerator>/<Denominator> or <Value>% */
	static bool ReadProgress(const TCHAR *&Text, int32 &OutNumerator, int32 &OutDenominator);

	/** Read an integer from the input stream */
	static bool ReadInteger(const TCHAR *&Text, uint32 &OutInteger);

	/** Read a string from the input stream */
	static bool ReadString(const TCHAR *&Text, FString &OutString);

	/** Skip over a sequence of whitespace characters in the input stream and returns a pointer to the next character */
	static const TCHAR *SkipWhitespace(const TCHAR *Text);
};
