// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Function.h"

/**
 * Helper to provide backward compatibility when converting a raw pointer into accessors.
 *
 * The helper is trying hard to mimic's a raw pointer field functionality without breaking compatibility for existing code.
 *
 * The helper's getter are all const and return a non-const pointer. This is because
 * accessing a pointer field inside a const struct will not make that pointer const like we typically
 * do for accessors. Since we want to mimic the behavior of a public field as much as possible, we
 * offer that same functionality.
 *
 * The helper supports being captured in a lamba as seen below and will become a const copy of the value at the capture's moment.
 * ex:
 *     [CapturedValue = Object.OnceARawFieldBecomingAnAccessor]()
 *     {
 *         // ExtractedValue contains a copy that will not change even if the original Object.OnceARawFieldBecomingAnAccessor changes.
 *         const Class* ExtractedValue = CapturedValue;
 *     });
 *
 * This helper also supports taking a nullptr in its constructor so it properly supports conditional like such
 * ex:
 *    ExtractedValue* Val = (Condition) ? Object.OnceARawFieldBecomingAnAccessor : nullptr;
 *
 *    A comforming compiler is supposed to try both conversions (nullptr -> TFieldPtrAccessor and TFieldPtrAccessor -> nullptr)
 *    but MSVC without the /permissive- flag will only try to cast nullptr into a TFieldPtrAccessor, which has to succeed to
 *    avoid breaking compabitility with existing code.
 *
 *    For more info, please refer to 
 *    https://docs.microsoft.com/en-us/cpp/build/reference/permissive-standards-conformance?view=vs-2019#ambiguous-conditional-operator-arguments
 *
 */
template <typename T>
class TFieldPtrAccessor
{
public:
	// Owned by another class that will control the value. Will not use the internal value.
	TFieldPtrAccessor(TFunction<T* ()> InGet, TFunction<void(T*)> InSet)
		: Get(MoveTemp(InGet))
		, Set(MoveTemp(InSet))
	{
	}

	// Self-owned value with initializer
	TFieldPtrAccessor(T* InValue = nullptr)
		: CapturedValue(InValue)
		, Get([this]()-> T* { return CapturedValue; })
		, Set([this](T* InPtr) { CapturedValue = InPtr; })
	{
	}

	// Capture the value of the passed field accessor and becomes self-owned.
	TFieldPtrAccessor(const TFieldPtrAccessor& Other)
		: CapturedValue(Other.Get())
		, Get([this]()-> T* { return CapturedValue; })
		, Set([this](T* InPtr) { CapturedValue = InPtr; })
	{
	}

	TFieldPtrAccessor& operator= (const TFieldPtrAccessor& Other) = delete;
	TFieldPtrAccessor& operator=(TFieldPtrAccessor&&) = delete;

	T* operator ->() const
	{
		return Get();
	}

	operator bool() const
	{
		return Get() != nullptr;
	}

	operator T* () const
	{
		return Get();
	}

	template<typename OtherT>
	explicit operator OtherT* () const
	{
		return (OtherT*)Get();
	}

	bool operator!() const
	{
		return Get() == nullptr;
	}

	bool operator == (const T* OtherPtr) const
	{
		return Get() == OtherPtr;
	}

	bool operator != (const T* OtherPtr) const
	{
		return Get() != OtherPtr;
	}

	TFieldPtrAccessor& operator= (T* OtherPtr)
	{
		Set(OtherPtr);
		return *this;
	}

private:
	// This is used only when capturing the value from another TFieldPtrAccessor.
	T* CapturedValue = nullptr;
	TFunction< T* ()> Get;
	TFunction<void(T*)> Set;
};