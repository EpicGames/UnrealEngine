// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "Math/Color.h"
#include "Math/MathFwd.h"
#include "Memory/MemoryFwd.h"
#include "Misc/EnumClassFlags.h"
#include "Tasks/Task.h"
#include "Templates/UnrealTemplate.h"

class FArchive;
class FText;
class IPlatformFile;
template <typename FuncType> class TFunctionRef;

/*-----------------------------------------------------------------------------
	FFileHelper
-----------------------------------------------------------------------------*/
struct FFileHelper
{
	enum class EHashOptions
	{
		None                =0,
		/** Enable the async task for verifying the hash for the file being loaded */
		EnableVerify		=1<<0,
		/** A missing hash entry should trigger an error */
		ErrorMissingHash	=1<<1
	};

	enum class EEncodingOptions
	{
		AutoDetect,
		ForceAnsi,
		ForceUnicode,
		ForceUTF8,
		ForceUTF8WithoutBOM
	};

	enum class EColorChannel
	{
		R,
		G,
		B,
		A,
		All
	};

	/** Options for methods that load files, for use with designated initializers */
	struct FLoadFileAsyncOptions
	{
		IPlatformFile* OverridePlatformFile			= nullptr;
		EAsyncIOPriorityAndFlags PriorityAndFlags 	= AIOP_Normal;
		IPlatformFile::EOpenReadFlags FileOpenFlags = IPlatformFile::EOpenReadFlags::None;
		EHashOptions VerifyFlags 					= EHashOptions::None;
	};
	/**
	 * Load a text file to an FString.
	 * Supports all combination of ANSI/Unicode files and platforms.
	*/
	static CORE_API void BufferToString( FString& Result, const uint8* Buffer, int32 Size );

	/**
	 * Load a binary file to a dynamic array with two uninitialized bytes at end as padding.
	 *
	 * @param Result    Receives the contents of the file
	 * @param Filename  The file to read
	 * @param Flags     Flags to pass to IFileManager::CreateFileReader
	*/
	static CORE_API bool LoadFileToArray( TArray<uint8>& Result, const TCHAR* Filename, uint32 Flags = 0 );

	/**
	 * Load a binary file to a dynamic array with two uninitialized bytes at end as padding.
	 *
	 * @param Result    Receives the contents of the file
	 * @param Filename  The file to read
	 * @param Flags     Flags to pass to IFileManager::CreateFileReader
	*/
	static CORE_API bool LoadFileToArray( TArray64<uint8>& Result, const TCHAR* Filename, uint32 Flags = 0 );

	/**
	 * Load a binary file a block at a time into a buffer and call the visitor each time the buffer is populated
	 * with a new block. This uses less memory and has higher cache coherency than loading the entire file into
	 * a block of bytes, and is more performant than reading the file a byte at a time. Useful for e.g. hashing or
	 * parsing.
	 * 
	 * @param Filename The file to read
	 * @param BlockVisitor Function called repeatedly with each sequential block.
	 *        Size of the input might change for performance and should not be assumed. Possibly the entire file will
	 *        be passed in a single block. If LoadFileInBlocks returns true, the sum of sizes in each call to
	 *        BlockVisitor equals the size of the requested range of the the file on disk. If the requested range
	 *        has size 0, LoadFileInBlocks will return true and the BlockVisitor will not be called.
	 * @param Offset Start of the range of bytes to read from the file. If less than zero or larger than filesize,
	 *        it will be silently clamped to the [0,FileSize] range. This may result in LoadFileInBlocks returning true
	 *        but using an empty range and not calling BlockVisitor.
	 * @param Length of the range of bytes to read from the file. If less than zero, all bytes between Offset and end
	 *        of file are read. If larger than the size of the range between Offset and end of file, it will be
	 *        silently shortened to the size of the range.
	 * @param Flags Flags to pass to IFileManager::CreateFileReader
	 * @param BlockSize Size of the internally-allocated buffer to read bytes into and pass to each call to
	 *        BlockVisitor. Sizes <= 0 use the default-selected block size for the current platform.
	 */
	static CORE_API bool LoadFileInBlocks(FStringView Filename, TFunctionRef<void(FMemoryView)> BlockVisitor,
		int64 Offset = 0, int64 Size = -1, uint32 Flags = 0, int64 BlockSize = 0);
	/**
	 * Loads a text file from an FArchive to an FString. Supports all combination of ANSI/Unicode files and platforms.
	 *
	 * @param Result       String representation of the loaded file
	 * @param Reader       The archive to load from
	 * @param VerifyFlags  Flags controlling the hash verification behavior ( see EHashOptions )
	 */
	static CORE_API bool LoadFileToString(FString& Result, FArchive& Reader, EHashOptions VerifyFlags = EHashOptions::None);

	/**
	 * Load a text file to an FString. Supports all combination of ANSI/Unicode files and platforms.
	 *
	 * @param Result       String representation of the loaded file
	 * @param Filename     Name of the file to load
	 * @param VerifyFlags  Flags controlling the hash verification behavior ( see EHashOptions )
	 */
	static CORE_API bool LoadFileToString( FString& Result, const TCHAR* Filename, EHashOptions VerifyFlags = EHashOptions::None, uint32 ReadFlags = 0 );

	/**
	 * Load a text file to an FString. Supports all combination of ANSI/Unicode files and platforms.
	 *
	 * @param Result       String representation of the loaded file
	 * @param PlatformFile PlatformFile interface to use
	 * @param Filename     Name of the file to load
	 * @param VerifyFlags  Flags controlling the hash verification behavior ( see EHashOptions )
	 */
	static CORE_API bool LoadFileToString(FString& Result, IPlatformFile* PlatformFile, const TCHAR* Filename, EHashOptions VerifyFlags = EHashOptions::None, uint32 ReadFlags = 0);

	/**
	 * Asynchronously load a text file to an FString. Supports all combination of ANSI/Unicode files and platforms.
	 *
	 * @param Filename     Name of the file to load
	 * @param Options	   Additional options to control behavior, use designated initializers for customization.
	 *
	 * e.g. 
	 * @code
	 * FFileHelper::LoadFileToStringAsync(TEXT("foo.txt"), {});
	 * FFileHelper::LoadFileToStringAsync(TEXT("foo.txt"),
	 * 		{ .OverridePlatformFile = MyPlatformFile, .FileOpenFlags = IPlatformFile::EOpenReadFlags::NoBuffering });
	 * @endcode  
	 */
	static CORE_API UE::Tasks::TTask<TValueOrError<FString, FFileSystemError>> LoadFileToStringAsync(const TCHAR* Filename, FLoadFileAsyncOptions Options);

	/**
	 * Load a text file to an array of strings. Supports all combination of ANSI/Unicode files and platforms.
	 *
	 * @param Result       String representation of the loaded file
	 * @param Filename     Name of the file to load
	 */
	static CORE_API bool LoadFileToStringArray( TArray<FString>& Result, const TCHAR* Filename );

	/**
	 * Load a text file to an array of strings, filtered by a user-defined predicate. Supports all combination of ANSI/Unicode files and platforms.
	 *
	 * @param Result       String representation of the loaded file
	 * @param Filename     Name of the file to load
	 * @param Predicate    Condition for whether or not to add the line to the array
	 */
	static CORE_API bool LoadFileToStringArrayWithPredicate(TArray<FString>& Result, const TCHAR* Filename, TFunctionRef<bool(const FString&)> Predicate);

	/**
	 * Load a text file and invoke a visitor for each line. Supports all combination of ANSI/Unicode files and platforms.
	 *
	 * @param Filename     Name of the file to load
	 * @param Visitor      Visitor to invoke for each non-empty line in the file
	 */
	static CORE_API bool LoadFileToStringWithLineVisitor(const TCHAR* Filename, TFunctionRef<void(FStringView Line)> Visitor);

	/**
	 * Save a binary array to a file.
	 */
	static CORE_API bool SaveArrayToFile(TArrayView64<const uint8> Array, const TCHAR* Filename, IFileManager* FileManager = &IFileManager::Get(), uint32 WriteFlags = 0);

	/**
	 * Save a binary array to a file.
	 */
	static CORE_API bool SaveArrayToFile( const TArray64<uint8>& Array, const TCHAR* Filename, IFileManager* FileManager = &IFileManager::Get(), uint32 WriteFlags = 0 );

	/**
	 * Write the FString to a file.
	 * Supports all combination of ANSI/Unicode files and platforms.
	 */
	static CORE_API bool SaveStringToFile( FStringView String, const TCHAR* Filename, EEncodingOptions EncodingOptions = EEncodingOptions::AutoDetect, IFileManager* FileManager = &IFileManager::Get(), uint32 WriteFlags = 0 );

	/**
	 * Write the FString to a file.
	 * Supports all combination of ANSI/Unicode files and platforms.
	 */
	static CORE_API bool SaveStringArrayToFile( const TArray<FString>& Lines, const TCHAR* Filename, EEncodingOptions EncodingOptions = EEncodingOptions::AutoDetect, IFileManager* FileManager = &IFileManager::Get(), uint32 WriteFlags = 0 );

	/**
	 * Saves a 24/32Bit BMP file to disk for debug image dump purposes
	 * 
	 * for general image saving (to BMP or any other format); use FImageUtils::SaveImage instead
	 * CreateBitmap is mainly for debug dump images
	 * 
	 * note this also calls SendDataToPCViaUnrealConsole
	 *   and uses GenerateNextBitmapFilename if Pattern does not have ".bmp" on it
	 * 
	 * @param Pattern filename with path, must not be 0, if with "bmp" extension (e.g. "out.bmp") the filename stays like this, if without (e.g. "out") automatic index numbers are addended (e.g. "out00002.bmp")
	 * @param DataWidth - Width of the bitmap supplied in Data >0
	 * @param DataHeight - Height of the bitmap supplied in Data >0
	 * @param Data must not be 0
	 * @param SubRectangle optional, specifies a sub-rectangle of the source image to save out. If NULL, the whole bitmap is saved
	 * @param FileManager must not be 0
	 * @param OutFilename optional, if specified filename will be output
	 * @param bInWriteAlpha optional, specifies whether to write out the alpha channel. Will force BMP V4 format.
	 * @param ColorChannel optional, specifies a specific channel to write out (will be written out to all channels gray scale).
	 *
	 * @return true if success
	 */
	static CORE_API bool CreateBitmap( const TCHAR* Pattern, int32 DataWidth, int32 DataHeight, const struct FColor* Data, FIntRect* SubRectangle = NULL, IFileManager* FileManager = &IFileManager::Get(), FString* OutFilename = NULL, bool bInWriteAlpha = false, EColorChannel ColorChannel = EColorChannel::All);

	/**
	 * Generates the next unique bitmap filename with a specified extension
	 *
	 * @param Pattern		Filename with path, but without extension.
	 * @param Extension		File extension to be appended
	 * @param OutFilename	Reference to an FString where the newly generated filename will be placed
	 * @param FileManager	Reference to a IFileManager (or the global instance by default)
	 *
	 * @return true if success
	 */
	static CORE_API bool GenerateNextBitmapFilename(const FString& Pattern, const FString& Extension, FString& OutFilename, IFileManager* FileManager = &IFileManager::Get());
	
	/**
	 * Generates the next unique bitmap filename with a specified extension
	 *
	 * @param Pattern		Filename with path, but without extension.
	 * @param Extension		File extension to be appended
	 * @param OutFilename	Reference to an FString where the newly generated filename will be placed
	 *
	 * @return true if success
	 */
	static CORE_API void GenerateDateTimeBasedBitmapFilename(const FString& Pattern, const FString& Extension, FString& OutFilename);
	
	/**
	 *	Load the given ANSI text file to an array of strings - one FString per line of the file.
	 *	Intended for use in simple text parsing actions
	 *
	 *	@param	InFilename			The text file to read, full path
	 *	@param	InFileManager		The filemanager to use - NULL will use &IFileManager::Get()
	 *	@param	OutStrings			The array of FStrings to fill in
	 *
	 *	@return	bool				true if successful, false if not
	 */
	static CORE_API bool LoadANSITextFileToStrings(const TCHAR* InFilename, IFileManager* InFileManager, TArray<FString>& OutStrings);

	/**
	* Checks to see if a filename is valid for saving.
	* A filename must be under FPlatformMisc::GetMaxPathLength() to be saved
	*
	* @param Filename	Filename, with or without path information, to check.
	* @param OutError	If an error occurs, this is the reason why
	*/
	static CORE_API bool IsFilenameValidForSaving(const FString& Filename, FText& OutError);
};

ENUM_CLASS_FLAGS(FFileHelper::EHashOptions)
