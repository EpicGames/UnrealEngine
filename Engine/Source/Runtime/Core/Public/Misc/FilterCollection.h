// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"
#include "Misc/IFilter.h"

/**
 *	A simple collection of Filters, with additional Filter specific functionality.
 */
template< typename ItemType >
class TFilterCollection
	: public TSharedFromThis< TFilterCollection< ItemType > >
{
public:

	/** 
	 * TFilterCollection destructor. Unregisters from all child filter changed delegates.
	 */
	~TFilterCollection()
	{
		for( auto Iterator = ChildFilters.CreateIterator(); Iterator; ++Iterator )
		{
			const TSharedPtr< IFilter< ItemType > >& Filter = *Iterator;

			if ( Filter.IsValid() )
			{
				Filter->OnChanged().RemoveAll( this );
			}
		}
	}

	/**
	 *	Adds the specified Filter to the collection
	 *
	 *	@param	Filter	The filter object to add to the collection
	 *	@return			The index in the collection at which the filter was added
	 */
	int32 Add( const TSharedPtr< IFilter< ItemType > >& Filter )
	{
		int32 ExistingIdx;
		if ( ChildFilters.Find(Filter, ExistingIdx) )
		{
			// The filter already exists, don't add a new one but return the index where it was found.
			return ExistingIdx;
		}

		if( Filter.IsValid() )
		{
			Filter->OnChanged().AddSP( this, &TFilterCollection::OnChildFilterChanged );
		}

		int32 Result = ChildFilters.Add( Filter );
		ChangedEvent.Broadcast();

		return Result;
	}
	
	/** 
	 *	Removes as many instances of the specified Filter as there are in the collection
	 *	
	 *	@param	Filter	The filter object to remove from the collection
	 *	@return			The number of Filters removed from the collection
	 */
	int32 Remove( const TSharedPtr< IFilter< ItemType > >& Filter )
	{
		if (Filter.IsValid())
		{
			Filter->OnChanged().RemoveAll(this);
		}

		int32 Result = ChildFilters.Remove( Filter );

		// Don't broadcast if the collection didn't change
		if (Result > 0)
		{
			ChangedEvent.Broadcast();
		}

		return Result;
	}

	/** 
	 *	Gets the filter at the specified index
	 *	
	 *	@param	Index	The index of the requested filter in the ChildFilters array.
	 *	@return			Filter at the specified index
	 */
	TSharedPtr< IFilter< ItemType > > GetFilterAtIndex( int32 Index )
	{
		return ChildFilters[Index];
	}

	/** Returns the number of Filters in the collection */
	UE_FORCEINLINE_HINT int32 Num() const
	{
		return ChildFilters.Num();
	}

	/** 
	 *	Returns whether the specified Item passes all of the filters in the collection
	 *
	 *	@param	InItem	The Item to check against all child filter restrictions
	 *	@return			Whether the Item passed all child filter restrictions
	 */
	bool PassesAllFilters( ItemType InItem ) const
	{
		for (const TSharedPtr<IFilter<ItemType>>& Filter : ChildFilters)
		{
			if( !Filter->PassesFilter( InItem ) )
			{
				return false;
			}
		}

		return true;
	}

	/** Broadcasts anytime the restrictions of any of the child Filters change */
	DECLARE_EVENT( TFilterCollection<ItemType>, FChangedEvent );
	FChangedEvent& OnChanged() { return ChangedEvent; }

	using RangedForIteratorType = TArray<TSharedPtr<IFilter<ItemType>>>::RangedForIteratorType;
	using RangedForConstIteratorType = TArray<TSharedPtr<IFilter<ItemType>>>::RangedForConstIteratorType;

	/** STL-like iterators to enable range-based for loop support. */
	[[nodiscard]] UE_REWRITE constexpr RangedForIteratorType      begin()       { return ChildFilters.begin(); }
	[[nodiscard]] UE_REWRITE constexpr RangedForConstIteratorType begin() const { return ChildFilters.begin(); }
	[[nodiscard]] UE_REWRITE constexpr RangedForIteratorType      end()         { return ChildFilters.end(); }
	[[nodiscard]] UE_REWRITE constexpr RangedForConstIteratorType end()   const { return ChildFilters.end(); }

protected:

	/** 
	 *	Called when a child Filter restrictions change and broadcasts the FilterChanged delegate
	 *	for the collection
	 */
	void OnChildFilterChanged()
	{
		ChangedEvent.Broadcast();
	}

	/** The array of child filters */
	TArray< TSharedPtr< IFilter< ItemType > > > ChildFilters;

	/**	Fires whenever any filter in the collection changes */
	FChangedEvent ChangedEvent;
};
