// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Char.h"


/** 
 * FNV hash generation for different types of input data
 **/
struct FFnv
{
	/** generates FNV hash of the memory area */
	static CORE_API uint32 MemFnv32( const void* Data, int32 Length, uint32 FNV=0 );
    static CORE_API uint64 MemFnv64( const void* Data, int32 Length, uint64 FNV=0 );
};
