// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformAffinity.h"

#ifndef DEFAULT_SERVER_FAKE_FORKS
	// Defines if the server process should always simulate forking even on platforms that do not support it.
	#define DEFAULT_SERVER_FAKE_FORKS 0
#endif

#ifndef DEFAULT_FORK_PROCESS_MULTITHREAD
	// Defines if forked child processes should support multithreading by default
	#define DEFAULT_FORK_PROCESS_MULTITHREAD 0
#endif

class FRunnable;
class FRunnableThread;

enum class EForkProcessRole : uint8
{
	Parent,
	Child,
};

/**
 * Helper functions for processes that fork in order to share memory pages.
 *
 * About multithreading:
 * When a process gets forked, any existing threads will not exist on the new forked process.
 * To solve this we use forkable threads that are notified when the fork occurs and will automatically convert themselves into real runnable threads.
 * On the master process, these forkable threads will be fake threads that are executed on the main thread and will block the critical path.
 *
 * Currently the game code is responsible for calling Fork on itself than calling FForkProcessHelper::OnForkingOccured to transform the forkable threads.
 * Ideally the fork point is done right after the game has loaded all the assets it wants to share so it can maximize the shared memory pool.
 * From the fork point any memory page that gets written into by a forked process will be transferred into a unique page for this process.
 * 
 */
class FForkProcessHelper 
{
public:

	/**
	 * Returns true if the server process was launched with the intention to fork.
	 * This could be a process on a fork-supported platform that will launch real child processes. (-WaitAndFork is set)
	 * Or it could be a process that will simulate forking by tranforming itself into a child process via fake forking (-FakeForking is set)
	 */
	static CORE_API bool IsForkRequested();

	/**
	 * Are we a forked process that supports multithreading
	 * This only becomes true after its safe to be multithread.
	 * Since a process can be forked mid-tick, there is a period of time where IsForkedChildProcess is true but IsForkedMultithreadInstance will be false
	 */
	static CORE_API bool IsForkedMultithreadInstance();

	/**
	 * Is this a process that was forked
	 */
	static CORE_API bool IsForkedChildProcess();

	/**
	 * Sets the forked child process flag and index given to this child process
	 */
	static CORE_API void SetIsForkedChildProcess(uint16 ChildIndex=1);

	/**
	* Returns the unique index of this forked child process. Index 0 is for the master server
	*/
	static CORE_API uint16 GetForkedChildProcessIndex();

	/**
	 * Event triggered when a fork occurred on the child process and its safe to create real threads
	 */
	static CORE_API void OnForkingOccured();

	/**
	 * Tells if we allow multithreading on forked processes.
	 * Default is set to false but can be configured to always be true via DEFAULT_MULTITHREAD_FORKED_PROCESSES
	 * Enabled via -PostForkThreading
	 * Disabled via -DisablePostForkThreading
	 */
	static CORE_API bool SupportsMultithreadingPostFork();

	/**
	 * Creates a thread according to the environment it's in:
	 *	In environments with SupportsMultithreading: create a real thread that will tick the runnable object itself
	 *	In environments without multithreading: create a fake thread that is ticked by the main thread.
	 *  In environments without multithreading but that allows multithreading post-fork: 
	 *		If called on the original master process: will create a forkable thread that is ticked in the main thread pre-fork but becomes a real thread post-fork
	 *      If called on a forked child process: will create a real thread immediately
	 */
	static CORE_API FRunnableThread* CreateForkableThread(
		class FRunnable* InRunnable,
		const TCHAR* InThreadName,
		uint32 InStackSize = 0,
		EThreadPriority InThreadPri = TPri_Normal,
		uint64 InThreadAffinityMask = FPlatformAffinity::GetNoAffinityMask(),
		EThreadCreateFlags InCreateFlags = EThreadCreateFlags::None,
		bool bAllowPreFork = false
	);

	/**
	 * Performs low-level cross-platform actions that should happen immediately BEFORE forking in a well-specified order.
	 * Runs after any higher level code like calling into game-level constructs or anything that may allocate memory.
	 * E.g. notifies GMalloc to optimize for memory sharing across parent/child process
	 * Note: This will be called multiple times on the parent before each fork. 
	 */
	static CORE_API void LowLevelPreFork();

	/**
	 * Performs low-level cross-platform actions that should happen immediately AFTER forking in the PARENT process in a well-specified order.
	 * Runs before any higher level code like calling into game-level constructs.
	 * E.g. notifies GMalloc to optimize for memory sharing across parent/child process
	 */
	static CORE_API void LowLevelPostForkParent();

	/**
	 * Performs low-level cross-platform actions that should happen immediately AFTER forking in the CHILD process in a well-specified order.
	 * Runs before any higher level code like calling into game-level constructs.
	 * E.g. notifies GMalloc to optimize for memory sharing across parent/child process
	 */
	static CORE_API void LowLevelPostForkChild(uint16 ChildIndex=1);
};



