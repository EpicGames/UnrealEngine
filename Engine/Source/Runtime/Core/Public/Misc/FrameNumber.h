// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/EnableIf.h"
#include "Templates/UnrealTypeTraits.h"

class FArchive;
template <typename NumericType> struct TNumericLimits;

/**
 * Typesafe 32-bit signed frame number. Defined in this way to prevent erroneous float->int conversions and afford type-safe operator overloading.
 */
struct FFrameNumber
{
	constexpr FFrameNumber()
		: Value(0)
	{}

	/**
	 * Implicit construction from a signed integer frame number, whilst disallowing any construction from other types.
	 */
	template<typename T, typename U = typename TEnableIf<std::is_same_v<T, int32>>::Type>
	constexpr FFrameNumber(T /*int32*/ InValue)
		: Value(InValue)
	{}

	/**
	 * Serializes the given FrameNumber from or into the specified archive.
	 *
	 * @param Ar            The archive to serialize from or into.
	 * @param FrameNumber   The bound to serialize.
	 * @return The archive used for serialization.
	 */
	friend CORE_API FArchive& operator<<(FArchive& Ar, FFrameNumber& FrameNumber);

	CORE_API bool Serialize(FArchive& Ar);

	FFrameNumber& operator+=(FFrameNumber RHS)                    { Value += RHS.Value; return *this; }
	FFrameNumber& operator-=(FFrameNumber RHS)                    { Value -= RHS.Value; return *this; }
	FFrameNumber& operator%=(FFrameNumber RHS)                    { Value %= RHS.Value; return *this; }

	FFrameNumber& operator++()                                    { ++Value; return *this; }
	FFrameNumber& operator--()                                    { --Value; return *this; }

	FFrameNumber operator++(int32)                                { FFrameNumber Ret = *this; ++Value; return Ret; }
	FFrameNumber operator--(int32)                                { FFrameNumber Ret = *this; --Value; return Ret; }

	friend bool operator==(FFrameNumber A, FFrameNumber B)        { return A.Value == B.Value; }
	friend bool operator!=(FFrameNumber A, FFrameNumber B)        { return A.Value != B.Value; }

	friend bool operator< (FFrameNumber A, FFrameNumber B)        { return A.Value < B.Value; }
	friend bool operator> (FFrameNumber A, FFrameNumber B)        { return A.Value > B.Value; }
	friend bool operator<=(FFrameNumber A, FFrameNumber B)        { return A.Value <= B.Value; }
	friend bool operator>=(FFrameNumber A, FFrameNumber B)        { return A.Value >= B.Value; }

	friend FFrameNumber operator+(FFrameNumber A, FFrameNumber B) { return FFrameNumber(A.Value + B.Value); }
	friend FFrameNumber operator-(FFrameNumber A, FFrameNumber B) { return FFrameNumber(A.Value - B.Value); }
	friend FFrameNumber operator%(FFrameNumber A, FFrameNumber B) { return FFrameNumber(A.Value % B.Value); }

	friend FFrameNumber operator-(FFrameNumber A)                 { return FFrameNumber(-A.Value); }

	friend FFrameNumber operator*(FFrameNumber A, float Scalar)   { return FFrameNumber(static_cast<int32>(FMath::Clamp(FMath::FloorToDouble(double(A.Value) * Scalar), (double)TNumericLimits<int32>::Min(), (double)TNumericLimits<int32>::Max()))); }
	friend FFrameNumber operator/(FFrameNumber A, float Scalar)   { return FFrameNumber(static_cast<int32>(FMath::Clamp(FMath::FloorToDouble(double(A.Value) / Scalar), (double)TNumericLimits<int32>::Min(), (double)TNumericLimits<int32>::Max()))); }

	friend inline uint32 GetTypeHash(FFrameNumber A)
	{
		return A.Value;
	}

	/**
	 * The value of the frame number
	 */
	int32 Value;
};

template<>
struct TNumericLimits<FFrameNumber>
{
	typedef FFrameNumber NumericType;

	static constexpr NumericType Min()
	{
		return MIN_int32;
	}

	static constexpr NumericType Max()
	{
		return MAX_int32;
	}

	static constexpr NumericType Lowest()
	{
		return Min();
	}
};