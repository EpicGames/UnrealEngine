// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/NumericLimits.h"
#include "Math/Range.h"
#include "Math/RangeBound.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "Templates/ValueOrError.h"

struct FExpressionError;

/**
 * A frame rate represented as a fraction comprising 2 integers: a numerator (number of frames), and a denominator (per second)
 */
struct FFrameRate
{
	/**
	 * Default construction to a frame rate of 60000 frames per second (0.0166 ms)
	 */
	FFrameRate()
		: Numerator(60000), Denominator(1)
	{}

	FFrameRate(uint32 InNumerator, uint32 InDenominator)
		: Numerator(InNumerator), Denominator(InDenominator)
	{}

	/** IMPORTANT: If you change the struct data, ensure that you also update the version in NoExportTypes.h  */

	/**
	 * The numerator of the framerate represented as a number of frames per second (e.g. 60 for 60 fps)
	 */
	int32 Numerator; 

	/**
	 * The denominator of the framerate represented as a number of frames per second (e.g. 1 for 60 fps)
	 */
	int32 Denominator;

	/**
	 * Verify that this frame rate is valid to use
	 */
	bool IsValid() const
	{
		return Denominator > 0;
	}

	/**
	 * Get the decimal representation of this framerate's interval
	 * 
	 * @return The time in seconds for a single frame under this frame rate
	 */
	double AsInterval() const;

	/**
	 * Get the decimal representation of this framerate
	 * 
	 * @return The number of frames per second
	 */
	double AsDecimal() const;

	/**
	 * Convert the specified frame number to a floating-point number of seconds based on this framerate
	 * 
	 * @param FrameNumber         The frame number to convert
	 * @return The number of seconds that the specified frame number represents
	 */
	double AsSeconds(FFrameTime FrameNumber) const;

	/**
	 * Convert the specified time in seconds to a frame number by rounding down to the nearest integer.
	 * 
	 * @param InTimeSeconds       The time to convert in seconds
	 * @return A frame number that represents the supplied time. Rounded down to the nearest integer.
	 */
	FFrameTime AsFrameTime(double InTimeSeconds) const;

	/**
	 * Convert the specified time in seconds to a frame number by rounding down to the nearest integer.
	 * 
	 * @param InTimeSeconds       The time to convert in seconds
	 * @return A frame number that represents the supplied time. Rounded down to the nearest integer.
	 */
	FFrameNumber AsFrameNumber(double InTimeSeconds) const;

	/**
	 * Check whether this frame rate is a multiple of another
	 */
	bool IsMultipleOf(FFrameRate Other) const;

	/**
	 * Check whether this frame rate is a factor of another
	 */
	bool IsFactorOf(FFrameRate Other) const;

	/**
	 * Convert the specified time from one framerate to another framerate
	 * 
	 * @param SourceTime         The frame number to convert
	 * @param SourceRate         The source frame rate
	 * @param DestinationRate    The destination frame rate
	 * @return A frame time in the destination frame rate
	 */
	static FFrameTime TransformTime(FFrameTime SourceTime, FFrameRate SourceRate, FFrameRate DestinationRate);

	/**
	 * Snap a time specified in one framerate, to another
	 * 
	 * @param SourceTime         The frame number to convert
	 * @param SourceRate         The source frame rate
	 * @param SnapToRate         The destination frame rate
	 * @return A frame time in the destination frame rate
	 */
	static FFrameTime Snap(FFrameTime SourceTime, FFrameRate SourceRate, FFrameRate SnapToRate);

	/**
	 * Convert this frame rate to a prettified text string.
	 * @note: Does not check against decorated frame rate names in FCommonFrameRates
	 */
	CORE_API FText ToPrettyText() const;

	/**
	 * Compute a desirable grid spacing for the specified screen units
	 * 
	 * @param PixelsPerSecond    The number of pixels representing a second of time
	 * @param OutMajorInterval   (Out) The interval in seconds at which to draw major grid lines
	 * @param OutMinorDivisions  (Out) The number of divisions to draw between major tick lines
	 * @param MinTickPx          (Optional) The smallest size in pixels that is desirable between ticks
	 * @param DesiredMajorTickPx (Optional) The desired size to compute major tick lines from
	 * @return True if a valid grid spacing was computed, false otherwise.
	 */
	CORE_API bool ComputeGridSpacing(const float PixelsPerSecond, double& OutMajorInterval, int32& OutMinorDivisions, float MinTickPx = 30.f, float DesiredMajorTickPx = 120.f) const;

	/**
	 * Get the maximum number of seconds representable with this framerate
	 */
	CORE_API double MaxSeconds() const;

	/**
	 * Get the reciprocal of this frame rate
	 */
	FFrameRate Reciprocal() const
	{
		return FFrameRate(Denominator, Numerator);
	}

	friend inline bool operator==(const FFrameRate& A, const FFrameRate& B)
	{
		return A.Numerator == B.Numerator && A.Denominator == B.Denominator;
	}

	friend inline bool operator!=(const FFrameRate& A, const FFrameRate& B)
	{
		return A.Numerator != B.Numerator || A.Denominator != B.Denominator;
	}

	friend inline FFrameRate operator*(FFrameRate A, FFrameRate B)
	{
		return FFrameRate(A.Numerator * B.Numerator, A.Denominator * B.Denominator);
	}

	friend inline FFrameRate operator/(FFrameRate A, FFrameRate B)
	{
		return FFrameRate(A.Numerator * B.Denominator, A.Denominator * B.Numerator);
	}

	friend inline double operator/(FFrameNumber Frame, FFrameRate Rate)
	{
		return Rate.AsSeconds(FFrameTime(Frame));
	}

	friend inline TRange<double> operator/(const TRange<FFrameNumber>& FrameRange, FFrameRate Rate)
	{
		TRangeBound<FFrameNumber> LowerBound = FrameRange.GetLowerBound();
		TRangeBound<FFrameNumber> UpperBound = FrameRange.GetUpperBound();

		return TRange<double>(
			LowerBound.IsOpen()
				? TRangeBound<double>::Open()
				: LowerBound.IsInclusive()
					? TRangeBound<double>::Inclusive(Rate.AsSeconds(LowerBound.GetValue()))
					: TRangeBound<double>::Inclusive(Rate.AsSeconds(LowerBound.GetValue()+1)),

			UpperBound.IsOpen()
				? TRangeBound<double>::Open()
				: UpperBound.IsInclusive()
					? TRangeBound<double>::Exclusive(Rate.AsSeconds(UpperBound.GetValue()+1))
					: TRangeBound<double>::Exclusive(Rate.AsSeconds(UpperBound.GetValue()))
			);
	}

	friend inline double operator/(FFrameTime FrameTime, FFrameRate Rate)
	{
		return Rate.AsSeconds(FrameTime);
	}

	friend inline FFrameTime operator*(double TimeInSeconds, FFrameRate Rate)
	{
		return Rate.AsFrameTime(TimeInSeconds);
	}

	friend inline FFrameTime operator*(float TimeInSeconds, FFrameRate Rate)
	{
		return Rate.AsFrameTime(TimeInSeconds);
	}

	friend inline uint32 GetTypeHash(const FFrameRate& Rate)
	{
		return HashCombine(GetTypeHash(Rate.Numerator), GetTypeHash(Rate.Denominator));
	}
	
	friend CORE_API FArchive& operator<<(FArchive& Ar, FFrameRate& FrameRate);

	CORE_API bool Serialize(FArchive& Ar);
};

inline double FFrameRate::AsInterval() const
{
	return double(Denominator) / double(Numerator);
}

inline double FFrameRate::AsDecimal() const
{
	return double(Numerator) / double(Denominator);
}

inline double FFrameRate::AsSeconds(FFrameTime FrameTime) const
{
	const int64  IntegerPart  = FrameTime.GetFrame().Value * int64(Denominator);
	const double FloatPart    = FrameTime.GetSubFrame()    * double(Denominator);

	return (double(IntegerPart) + FloatPart) / Numerator;
}

inline FFrameTime FFrameRate::AsFrameTime(double TimeInSeconds) const
{
	// @todo: sequencer-timecode: proper large number integer multiplication/division before coercion to float ?
	const double TimeAsFrame = (TimeInSeconds * Numerator) / Denominator;
	FFrameNumber FrameNumber = static_cast<int32>(FMath::Clamp(FMath::FloorToDouble(TimeAsFrame), static_cast<double>(TNumericLimits<int32>::Min()), static_cast<double>(TNumericLimits<int32>::Max())));

	float SubFrame = static_cast<float>(TimeAsFrame - FMath::FloorToDouble(TimeAsFrame));
	const int32 TruncatedSubFrame = FMath::TruncToInt(SubFrame);
	SubFrame -= static_cast<float>(TruncatedSubFrame);
	FrameNumber.Value += TruncatedSubFrame;
	if (SubFrame > 0.f)
	{
		SubFrame = FMath::Min(SubFrame, FFrameTime::MaxSubframe);
	}

	return FFrameTime(FrameNumber, SubFrame);
}

inline FFrameNumber FFrameRate::AsFrameNumber(double TimeInSeconds) const
{
	// @todo: sequencer-timecode: proper large number integer multiplication/division before coercion to float ?
	const double TimeAsFrame = (static_cast<double>(TimeInSeconds) * Numerator) / Denominator;
	return static_cast<int32>(FMath::Clamp(FMath::FloorToDouble(TimeAsFrame + static_cast<double>(FMath::TruncToInt(static_cast<float>(TimeAsFrame - FMath::FloorToDouble(TimeAsFrame))))), static_cast<double>(TNumericLimits<int32>::Min()), static_cast<double>(TNumericLimits<int32>::Max())));
}

inline FFrameTime ConvertFrameTime(FFrameTime SourceTime, FFrameRate SourceRate, FFrameRate DestinationRate)
{
	if (SourceRate == DestinationRate)
	{
		return SourceTime;
	}
	//We want NewTime =SourceTime * (DestinationRate/SourceRate);
	//And want to limit conversions and keep int precision as much as possible

	// If SourceRate.Numerator is 0 then NewDenominator will be 0 and will cause a divide by 0 error.
	check(SourceRate.Numerator != 0);

	//@todo: These integers should not need the volatile keyword here, but adding it works around
	//       a compiler bug that results in an uninitialized vector register being used
	volatile int64 NewNumerator = static_cast<int64>(DestinationRate.Numerator) * SourceRate.Denominator;
	volatile int64 NewDenominator = static_cast<int64>(DestinationRate.Denominator) * SourceRate.Numerator;

	double NewNumerator_d = double(NewNumerator);
	double NewDenominator_d = double(NewDenominator);
	//Now the IntegerPart may have a Float Part, and then the FloatPart may have an IntegerPart,
	//So we add the extra Float from the IntegerPart to the FloatPart and then add back any extra Integer to IntegerPart
	int64  IntegerPart = ( (int64)(SourceTime.GetFrame().Value) * NewNumerator ) / NewDenominator; //-V567
	const double IntegerFloatPart = ((double(SourceTime.GetFrame().Value) * double(NewNumerator)) / double(NewDenominator)) - double(IntegerPart);  //-V567
	const double FloatPart = ((SourceTime.GetSubFrame()    * NewNumerator_d) / NewDenominator_d) + IntegerFloatPart;
	const double FloatPartFloored = FMath::FloorToDouble(FloatPart);
	const int64 FloatAsInt = int64(FloatPartFloored);
	IntegerPart += FloatAsInt;
	float SubFrame = static_cast<float>(FloatPart - FloatPartFloored);
	if (SubFrame > 0.f)
	{
		SubFrame = FMath::Min(SubFrame, FFrameTime::MaxSubframe);
	}

	IntegerPart = FMath::Clamp<int64>(IntegerPart,TNumericLimits<int32>::Min(),TNumericLimits<int32>::Max());
	return FFrameTime(static_cast<int32>(IntegerPart), SubFrame);
}

inline FFrameTime FFrameRate::TransformTime(FFrameTime SourceTime, FFrameRate SourceRate, FFrameRate DestinationRate)
{
	return ConvertFrameTime(SourceTime, SourceRate, DestinationRate);
}

inline FFrameTime FFrameRate::Snap(FFrameTime SourceTime, FFrameRate SourceRate, FFrameRate SnapToRate)
{
	return ConvertFrameTime(ConvertFrameTime(SourceTime, SourceRate, SnapToRate).RoundToFrame(), SnapToRate, SourceRate);
}

inline bool FFrameRate::IsMultipleOf(FFrameRate Other) const
{
	int64 CommonValueA = int64(Numerator) * Other.Denominator;
	int64 CommonValueB = int64(Other.Numerator) * Denominator;

	return CommonValueA <= CommonValueB && CommonValueB % CommonValueA == 0;
}

inline bool FFrameRate::IsFactorOf(FFrameRate Other) const
{
	return Other.IsMultipleOf(*this);
}

/**
 * Attempt to parse a frame rate from a string
 * Note: user code may need to #include "Misc/ExpressionParserTypes.h"
 */
CORE_API TValueOrError<FFrameRate, FExpressionError> ParseFrameRate(const TCHAR* FrameRateString);

/**
 * Common Lex::TryParseString overload for FFrameRate
 */
CORE_API bool TryParseString(FFrameRate& OutFrameRate, const TCHAR* InString);
