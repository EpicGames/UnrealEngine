// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameNumber.h"
#include "Misc/AssertionMacros.h"
#include "Templates/EnableIf.h"
#include "Containers/UnrealString.h"

/**
 * Structure representing a time by a context-free frame number, plus a sub frame value in the range [0:1)
 * Conversion to and from time in seconds is achieved in combination with FFrameRate.
 * Only the frame number part of this representation can be negative, sub frames are always a positive value between the frame number and its next logical frame
 */
struct FFrameTime
{

	static CORE_API const float MaxSubframe;

	/**
	 * Default constructor initializing to zero
	 */
	FFrameTime();

	/**
	 * Implicit construction from a single integer, while disallowing implicit conversion from any other numeric type
	 */
	template<typename T, typename = typename TEnableIf<std::is_same_v<T, int32>>::Type>
	FFrameTime(T /* int32 */ InFrameNumber);

	/**
	 * Implicit construction from a type-safe frame number
	 */
	FFrameTime(FFrameNumber InFrameNumber);

	/**
	 * Construction from a frame number and a sub frame
	 */
	FFrameTime(FFrameNumber InFrameNumber, float InSubFrame);

	/**
	 * Assignment from a type-safe frame number
	 */
	FFrameTime& operator=(FFrameNumber InFrameNumber);

	/**
	 * Serializes the given FrameTime from or into the specified archive
	 */
	bool Serialize(FArchive& Ar);

public:

	/**
	 * Access this time's frame number
	 */
	UE_FORCEINLINE_HINT FFrameNumber GetFrame() const
	{
		return FrameNumber;
	}

	/**
	 * Access this time's sub frame
	 */
	UE_FORCEINLINE_HINT float GetSubFrame() const
	{
		return SubFrame;
	}

	/**
	 * Return the first frame number less than or equal to this frame time
	 */
	FFrameNumber FloorToFrame() const;

	/**
	 * Return the next frame number greater than or equal to this frame time
	 */
	FFrameNumber CeilToFrame() const;

	/**
	 * Round to the nearest frame number
	 */
	FFrameNumber RoundToFrame() const;

	/**
	 * Retrieve a decimal representation of this frame time
	 * Sub frames are always added to the current frame number, so for negative frame times, a time of -10 [sub frame 0.25] will yield a decimal value of -9.75.
	 */
	double AsDecimal() const;

	/**
	 * Convert a decimal representation to a frame time
	 * Note that sub frames are always positive, so negative decimal representations result in an inverted sub frame and floored frame number
	 */
	static FFrameTime FromDecimal(double InDecimalFrame);

	/** IMPORTANT: If you change the struct data, ensure that you also update the version in NoExportTypes.h  */

	FFrameNumber FrameNumber;

private:

	/** Must be 0.f <= SubFrame < 1.f */
	float SubFrame;

public:

	/**
	 * Serializes the given FrameTime from or into the specified archive.
	 *
	 * @param Ar            The archive to serialize from or into.
	 * @param FrameTime     The frame time to serialize.
	 * @return The archive used for serialization.
	 */
	friend FArchive& operator<<(FArchive& Ar, FFrameTime& FrameTime)
	{
		FrameTime.Serialize(Ar);
		return Ar;
	}

	friend UE_FORCEINLINE_HINT bool operator==(FFrameTime A, FFrameTime B)
	{
		return A.FrameNumber == B.FrameNumber && A.SubFrame == B.SubFrame;
	}


	friend UE_FORCEINLINE_HINT bool operator!=(FFrameTime A, FFrameTime B)
	{
		return A.FrameNumber != B.FrameNumber || A.SubFrame != B.SubFrame;
	}


	friend UE_FORCEINLINE_HINT bool operator> (FFrameTime A, FFrameTime B)
	{
		return A.FrameNumber >  B.FrameNumber || ( A.FrameNumber == B.FrameNumber && A.SubFrame > B.SubFrame );
	}


	friend UE_FORCEINLINE_HINT bool operator>=(FFrameTime A, FFrameTime B)
	{
		return A.FrameNumber > B.FrameNumber || ( A.FrameNumber == B.FrameNumber && A.SubFrame >= B.SubFrame );
	}


	friend UE_FORCEINLINE_HINT bool operator< (FFrameTime A, FFrameTime B)
	{
		return A.FrameNumber <  B.FrameNumber || ( A.FrameNumber == B.FrameNumber && A.SubFrame < B.SubFrame );
	}


	friend UE_FORCEINLINE_HINT bool operator<=(FFrameTime A, FFrameTime B)
	{
		return A.FrameNumber < B.FrameNumber || ( A.FrameNumber == B.FrameNumber && A.SubFrame <= B.SubFrame );
	}


	friend inline FFrameTime& operator+=(FFrameTime& LHS, FFrameTime RHS)
	{
		float       NewSubFrame    = LHS.SubFrame + RHS.SubFrame;
		const int64 NewFrameNumber = int64(LHS.FrameNumber.Value) + int64(RHS.FrameNumber.Value) + FMath::FloorToInt(NewSubFrame);

		LHS.FrameNumber = static_cast<int32>(NewFrameNumber);
		LHS.SubFrame    = FMath::Frac(NewSubFrame);

		return LHS;
	}


	friend inline FFrameTime operator+(FFrameTime A, FFrameTime B)
	{
		const float        NewSubFrame    = A.SubFrame + B.SubFrame;
		const int64        NewFrameNumber = int64(A.FrameNumber.Value) + int64(B.FrameNumber.Value) + FMath::FloorToInt(NewSubFrame);

		return FFrameTime(static_cast<int32>(NewFrameNumber), FMath::Frac(NewSubFrame));
	}


	friend inline FFrameTime& operator-=(FFrameTime& LHS, FFrameTime RHS)
	{
		// Ensure SubFrame is always between 0 and 1
		// Note that the difference between frame -1.5 and 1.5 is 2, not 3, since sub frame positions are always positive
		const float        NewSubFrame     = LHS.SubFrame - RHS.SubFrame;
		const float        FlooredSubFrame = FMath::FloorToFloat(NewSubFrame);
		const int64        NewFrameNumber  = int64(LHS.FrameNumber.Value) - int64(RHS.FrameNumber.Value) + FMath::TruncToInt(FlooredSubFrame);

		LHS.FrameNumber.Value  = static_cast<int32>(NewFrameNumber);
		LHS.SubFrame = NewSubFrame - FlooredSubFrame;

		return LHS;
	}


	friend inline FFrameTime operator-(FFrameTime A, FFrameTime B)
	{
		// Ensure SubFrame is always between 0 and 1
		// Note that the difference between frame -1.5 and 1.5 is 2, not 3, since sub frame positions are always positive
		const float        NewSubFrame     = A.SubFrame - B.SubFrame;
		const float        FlooredSubFrame = FMath::FloorToFloat(NewSubFrame);
		const int64        NewFrameNumber  = int64(A.FrameNumber.Value) - int64(B.FrameNumber.Value) + FMath::TruncToInt(FlooredSubFrame);

		return FFrameTime(static_cast<int32>(NewFrameNumber), NewSubFrame - FlooredSubFrame);
	}


	friend inline FFrameTime operator%(FFrameTime A, FFrameTime B)
	{
		check(B.FrameNumber.Value != 0 || B.GetSubFrame() != 0.f);

		if (A.SubFrame == 0.f && B.SubFrame == 0.f)
		{
			return FFrameTime(A.FrameNumber % B.FrameNumber);
		}
		else
		{
			FFrameTime Result = A;
			while (Result >= B)
			{
				Result = Result - B;
			}
			return Result;
		}
	}


	friend inline FFrameTime operator-(FFrameTime A)
	{
		return A.GetSubFrame() == 0.f
			? FFrameTime(-A.FrameNumber)
			: FFrameTime(-A.FrameNumber - 1, 1.f-A.GetSubFrame());
	}


	friend UE_FORCEINLINE_HINT FFrameTime operator*(FFrameTime A, double Scalar)
	{
		return FFrameTime::FromDecimal(A.AsDecimal() * Scalar);
	}

	friend UE_FORCEINLINE_HINT FFrameTime operator*(double Scalar, FFrameTime A)
	{
		return FFrameTime::FromDecimal(A.AsDecimal() * Scalar);
	}

	friend UE_FORCEINLINE_HINT FFrameTime operator/(FFrameTime A, double Scalar)
	{
		return FFrameTime::FromDecimal(A.AsDecimal() / Scalar);
	}
};


inline FFrameTime::FFrameTime()
	: FrameNumber(0), SubFrame(0.f)
{}


template<typename T, typename>
FFrameTime::FFrameTime(T InFrameNumber)
	: FrameNumber(InFrameNumber), SubFrame(0.f)
{}


inline FFrameTime::FFrameTime(FFrameNumber InFrameNumber)
	: FrameNumber(InFrameNumber), SubFrame(0.f)
{}


inline FFrameTime::FFrameTime(FFrameNumber InFrameNumber, float InSubFrame)
	: FrameNumber(InFrameNumber), SubFrame(InSubFrame)
{
	// Hack to ensure that SubFrames are in a sensible range of precision to work around
	// problems with FloorToXYZ returning the wrong thing for very small negative numbers
	SubFrame = FMath::Clamp(InSubFrame + 0.5f - 0.5f, 0.f, MaxSubframe);
	checkSlow(SubFrame >= 0.f && SubFrame < 1.f);
}


inline bool FFrameTime::Serialize(FArchive& Ar)
{
	Ar << FrameNumber;
	Ar << SubFrame;
	return true;
}


inline FFrameTime& FFrameTime::operator=(FFrameNumber InFrameNumber)
{
	FrameNumber = InFrameNumber;
	SubFrame    = 0.f;
	return *this;
}

UE_FORCEINLINE_HINT FFrameNumber FFrameTime::FloorToFrame() const
{
	return FrameNumber;
}


UE_FORCEINLINE_HINT FFrameNumber FFrameTime::CeilToFrame() const
{
	return SubFrame == 0.f ? FrameNumber : FrameNumber+1;
}


UE_FORCEINLINE_HINT FFrameNumber FFrameTime::RoundToFrame() const
{
	return SubFrame < .5f ? FrameNumber : FrameNumber+1;
}


UE_FORCEINLINE_HINT double FFrameTime::AsDecimal() const
{
	return double(FrameNumber.Value) + SubFrame;
}

inline FFrameTime FFrameTime::FromDecimal(double InDecimalFrame)
{
	int32 NewFrame = static_cast<int32>(FMath::Clamp(FMath::FloorToDouble(InDecimalFrame), (double)TNumericLimits<int32>::Min(), (double)TNumericLimits<int32>::Max()));

	// Ensure fractional parts above the highest sub frame float precision do not round to 0.0
	double Fraction = InDecimalFrame - FMath::FloorToDouble(InDecimalFrame);
	return FFrameTime(NewFrame, FMath::Clamp((float)Fraction, 0.0f, MaxSubframe));
}

/** Convert a FFrameTime into a string */
inline FString LexToString(const FFrameTime InTime)
{
	return FString::Printf(TEXT("Frame: %d Subframe: %f"), InTime.GetFrame().Value, InTime.GetSubFrame());
}

/** Convert a FFrameTime into a string */
inline FString LexToShortString(const FFrameTime InTime)
{
	return FString::Printf(TEXT("%d.%f"), InTime.GetFrame().Value, InTime.GetSubFrame());
}

