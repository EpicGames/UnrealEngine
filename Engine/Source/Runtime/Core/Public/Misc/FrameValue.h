// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreGlobals.h"
#include "Misc/Optional.h"

/**
 * This struct allows you to cache a value for a frame, and automatically invalidates
 * when the frame advances.
 * If the value was set this frame, IsSet() returns true and GetValue() is valid.
 */
template<typename ValueType>
struct TFrameValue
{
private:
	uint64 FrameSet;
	TOptional<ValueType> Value;

public:
	/** Construct an OptionalType with a valid value. */
	TFrameValue(const ValueType& InValue)
		: FrameSet(GFrameCounter)
		, Value(InValue)
	{
	}

	TFrameValue(ValueType&& InValue)
		: FrameSet(GFrameCounter)
		, Value(MoveTemp(InValue))
	{
	}

	/** Construct an OptionalType with no value; i.e. unset */
	TFrameValue()
		: FrameSet(GFrameCounter)
	{
	}

	/** Copy/Move construction */
	TFrameValue(const TFrameValue& InValue)
		: FrameSet(GFrameCounter)
	{
		Value = InValue.Value;
	}

	TFrameValue(TFrameValue&& InValue)
		: FrameSet(GFrameCounter)
	{
		Value = MoveTemp(InValue.Value);
	}

	TFrameValue& operator=(const TFrameValue& InValue)
	{
		Value = InValue.Value;
		FrameSet = GFrameCounter;
		return *this;
	}

	TFrameValue& operator=(TFrameValue&& InValue)
	{
		Value = MoveTemp(InValue.Value);
		FrameSet = GFrameCounter;
		return *this;
	}

	TFrameValue& operator=(const ValueType& InValue)
	{
		Value = InValue;
		FrameSet = GFrameCounter;
		return *this;
	}

	TFrameValue& operator=(ValueType&& InValue)
	{
		Value = MoveTemp(InValue);
		FrameSet = GFrameCounter;
		return *this;
	}

public:

	bool IsSet() const
	{
		return Value.IsSet() && FrameSet == GFrameCounter;
	}

	const ValueType& GetValue() const
	{
		checkf(FrameSet == GFrameCounter, TEXT("Cannot get value on a different frame"));
		return Value.GetValue();
	}

	ValueType TryGetValue(ValueType UnsetValue) const&
	{
		return IsSet() ? Value.GetValue() : MoveTemp(UnsetValue);
	}

	ValueType TryGetValue(ValueType UnsetValue) &&
	{
		ValueType Result = IsSet() ? MoveTemp(Value.GetValue()) : MoveTemp(UnsetValue);
		Value.Reset();
		return Result;
	}
};