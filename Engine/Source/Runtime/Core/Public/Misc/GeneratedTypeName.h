// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/IntegerSequence.h"

#if defined(_MSC_VER) && !defined(__clang__)
	#define SIG __FUNCSIG__
	#define SIG_STARTCHAR '<'
	#define SIG_ENDCHAR '>'

	#pragma warning(push)
	#pragma warning(disable : 4503) // "identifier" : decorated name length exceeded, name was truncated
#else
	#define SIG __PRETTY_FUNCTION__
	#define SIG_STARTCHAR '='
	#define SIG_ENDCHAR ']'
#endif

#if !UE_BUILD_DOCS

namespace UETypeName_Private
{
	// Class representing a substring of another string
	struct FSubstr
	{
		const char* Ptr;
		uint32 Length;

		constexpr char operator[](uint32 Index) const
		{
			return Ptr[Index];
		}

		constexpr uint32 Len() const
		{
			return Length;
		}

		constexpr FSubstr PopFront(int Num = 1) const
		{
			return { Ptr + Num, Len() - Num };
		}

		constexpr FSubstr PopBack(int Num = 1) const
		{
			return { Ptr, Len() - Num };
		}

		constexpr FSubstr PopFrontAll(char Ch) const
		{
			return (Len() > 0 && Ptr[0] == Ch) ? PopFront().PopFrontAll(Ch) : *this;
		}

		constexpr FSubstr PopFrontAllNot(char Ch) const
		{
			return (Len() > 0 && Ptr[0] != Ch) ? PopFront().PopFrontAllNot(Ch) : *this;
		}

		constexpr FSubstr PopBackAll(char Ch) const
		{
			return (Len() > 0 && Ptr[Len() - 1] == Ch) ? PopBack().PopBackAll(Ch) : *this;
		}

		constexpr FSubstr PopBackAllNot(char Ch) const
		{
			return (Len() > 0 && Ptr[Len() - 1] != Ch) ? PopBack().PopBackAllNot(Ch) : *this;
		}
	};

	// Gets the name of the type as a substring of a literal
	template <typename T>
	constexpr FSubstr GetTypeSubstr()
	{
		return FSubstr{ SIG, sizeof(SIG) - 1 }
			.PopFrontAllNot(SIG_STARTCHAR)
			.PopBackAllNot(SIG_ENDCHAR)
			.PopFront()
			.PopBack()
			.PopFrontAll(' ')
			.PopBackAll(' ');
	}

	template <uint32 NumChars>
	struct TCharArray
	{
		TCHAR Array[NumChars + 1];
	};

	template <typename T, uint32... Indices>
	constexpr TCharArray<sizeof...(Indices)> TypeSubstrToCharArray(TIntegerSequence<uint32, Indices...>)
	{
		return TCharArray<sizeof...(Indices)> { { (TCHAR)GetTypeSubstr<T>()[Indices]... } };
	}
}

#endif

/**
 * Returns a pointer to a static string representing the name of the type, e.g.:
 *
 * const TCHAR* FStringName = GetGeneratedTypeName<FString>() // "FString"
 *
 * Caveats:
 * - The strings are compiler-dependent and thus non-portable, and so shouldn't be saved or relied upon as a form of identity, e.g. the example above returns "class FString" on MSVC.
 * - Default template parameters are also handled differently by different compilers, sometimes ignored, sometimes not.
 * - Only the concrete type is known to the compiler, aliases are ignored, e.g. GetGeneratedTypeName<TCHAR>() typically returns "wchar_t".
 */
template <typename T>
inline const TCHAR* GetGeneratedTypeName()
{
	static constexpr auto Result = UETypeName_Private::TypeSubstrToCharArray<T>(TMakeIntegerSequence<uint32, UETypeName_Private::GetTypeSubstr<T>().Len()>());
	return Result.Array;
}

#if defined(_MSC_VER) && !defined(__clang__)
	#pragma warning(pop)
#endif

#undef SIG_ENDCHAR
#undef SIG_STARTCHAR
#undef SIG
