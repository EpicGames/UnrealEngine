// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GrowableAllocator.h: Memory allocator that allocates direct memory for pool memory
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "Stats/Stats.h"

//////////////
// - Don't use a shared CriticalSection, must pass in to the GrowableAllocator
// - Use this with GPU allocations
// - search for @todo here and in the SwitchAudioBuffer code
// - Move this into the FSwitchAudioDevice
// - Pass this over some code reviews on slack or something for maybe a better way to structure instead of Template and Pure virtual
// - Can maybe just make virtuals in FGrowableAllocationBase ??
/////////////

#define ALLOCATION_HISTOGRAM			(!UE_BUILD_SHIPPING && !UE_BUILD_TEST) && 1
#define ALLOCATION_HISTOGRAM_DETAILED	(!UE_BUILD_SHIPPING && !UE_BUILD_TEST) && 0

struct FGrowableAllocationBase
{
	uint64 Size;
	uint32 Offset;
	uint32 Padding;
#if !UE_BUILD_SHIPPING
	uint32 OwnerType;
#endif
};

// a base class for both the classes below, for usage tracking only
class FGrowableMallocBase
{
public:
	/** Constructor */
	FGrowableMallocBase()
		: TotalAllocated(0)
		, TotalWaste(0)
		, CurrentAllocs(0)
		, TotalAllocs(0)
	{}
	virtual ~FGrowableMallocBase() = default;

	/** 
	 * Returns approximated amount of memory wasted due to allocations' alignment. 
	 */
	virtual uint64 GetWasteApproximation()
	{
		double Waste = (static_cast<double>(TotalWaste) / static_cast<double>(TotalAllocs)) * static_cast<double>(CurrentAllocs);
		return (uint64)Waste;
	}

protected:

	/** Total amount ever allocated */
	uint64			TotalAllocated;

	/** The total amount of memory wasted due to allocations' alignment. */
	uint64			TotalWaste;

	/** The current number of allocations. */
	uint64			CurrentAllocs;

	/** The total number of allocations. */
	uint64			TotalAllocs;
};



class FGrowableMallocChunk : public FGrowableMallocBase
{
public:
	// Abstract functions that user must implement

	/**
	 * Lets the implementation allocate the backing memory for the chunk
	 * 
	 * Implementation needs to pause LLM tracking for ELLMTracker::Default if it's allocating from an allocator that tracks via ELLMTracker::Default
	 *
	 * @param Size Minimum size needed for this allocation. The implementation will likely allocate more, and return that amount
	 * @return The actual size of the chunk that was allocated (could be much larger than Size)
	 */
	virtual uint64 CreateInternalMemory(uint64 Size) = 0;

	/**
	 * Destroys the backing memory for the chunk
	 *
	 * Implementation needs to pause LLM tracking for ELLMTracker::Default if it's allocating from an allocator that tracks via ELLMTracker::Default
	 *
	 */
	virtual void DestroyInternalMemory() = 0;

	/**
	 * Creates an implementation specific subclass of FGrowableAllocationBase. Does not need to be initialized (see InitializeAllocationStruct)
	 */
	virtual FGrowableAllocationBase* CreateAllocationStruct() = 0;

	/**
	 * Destroys the implemtnation object. By default, just deletes it
	 */
	virtual void DestroyAllocationStruct(FGrowableAllocationBase* Allocation)
	{
		delete Allocation;
	}

	/**
	 * Lets the implementation fill in any specific fields of the allocation struct after the base fields are set up
	 *
	 */
	virtual void InitializeAllocationStruct(FGrowableAllocationBase* Allocation) = 0;

	/**
	 * Queries the implementation if the given allocation came from this chunk
	 */
	virtual bool DoesChunkContainAllocation(const FGrowableAllocationBase* Allocation) = 0;

	/**
	 * Queries the implementation if the given address came from this chunk
	 */
	virtual bool DoesChunkContainAddress(const void* Address) = 0;


	/**
	* Constructor
	*/
	FGrowableMallocChunk(uint64 InSize, uint32 Type, FCriticalSection* InCriticalSection)
		: MemoryType(Type)
		, HeapSize(InSize)
		, UsedMemorySize(0)
		, MaxFreeEntrySize(MaxFreeEntrySizeDirty)
		, CriticalSection(InCriticalSection)
	{
	}

	void Initialize()
	{
		// create the pool object (note that this will update and return the new HeapSize for the implementations internal aligned size - we then update how big we track)
		HeapSize = CreateInternalMemory(HeapSize);
		// entire chunk is free
		FreeList = new FFreeEntry(NULL, 0, HeapSize);
		MaxFreeEntrySize = FreeList->BlockSize;
	}

	/**
	* Destructor
	*/
	virtual ~FGrowableMallocChunk() = default;

	void Destroy()
	{
		checkf(IsEmpty(), TEXT("Chunk was not empty when it was destroyed!"));
		DestroyInternalMemory();
	}

	/**
	* Check free list for an entry big enough to fit the requested Size with Alignment
	* @param Size - allocation size
	* @param Alignment - allocation alignment
	* @return true if available entry was found
	*/
	bool CanFitEntry(uint32 Size, uint32 Alignment)
	{
		// Compute MaxFreeEntrySize if necessary (should only happen if this chunk was just allocated from)
		if (MaxFreeEntrySize == MaxFreeEntrySizeDirty)
		{
			MaxFreeEntrySize = 0;
			for (FFreeEntry* Entry = FreeList; Entry; Entry = Entry->Next)
			{
				MaxFreeEntrySize = FMath::Max(Entry->BlockSize, MaxFreeEntrySize);
			}
		}

		// Return false if we trivially don't fit
		if (Size > MaxFreeEntrySize)
		{
			return false;
		}

		// Return true if we trivially do fit
		if (Size + Alignment - 1 <= MaxFreeEntrySize)
		{
			return true;
		}

		// Slow method - search the free entries for a free chunk
		bool bResult = false;
		for (FFreeEntry *Entry = FreeList; Entry; Entry = Entry->Next)
		{
			if (Entry->CanFit(Size, Alignment))
			{
				bResult = true;
				break;
			}
		}
		return bResult;
	}
	/**
	* @return true if this chunk has no used memory
	*/
	bool IsEmpty()
	{
		return UsedMemorySize == 0;
	}

	FGrowableAllocationBase* Malloc(uint32 Size, uint32 Alignment, uint32 MinAllocationSize, int32 OwnerType)
	{
		checkSlow(Alignment != 0);

		// multi-thread protection
		FScopeLock ScopeLock(CriticalSection);

		// Alignment here is assumed to be for location and size
		const uint32 AlignedSize = Align<uint32>(Size, Alignment);

		// Update stats.
		const uint32 WastedSize = AlignedSize - Size;
		TotalWaste += WastedSize;
		CurrentAllocs++;
		TotalAllocs++;

		// look for a good free chunk
		FFreeEntry* Prev = NULL;
		FFreeEntry* FindEntry = NULL;

		for (FFreeEntry* Entry = FreeList; Entry; Entry = Entry->Next)
		{
			if (Entry->CanFit(AlignedSize, Alignment))
			{
				FindEntry = Entry;
				break;
			}
			Prev = Entry;
		}

		if (FindEntry != NULL)
		{
			if (FindEntry->BlockSize == MaxFreeEntrySize)
			{
				// We're probably about to just split our largest entry, so mark the max size as dirty to indicate it needs recomputing
				MaxFreeEntrySize = MaxFreeEntrySizeDirty;
			}
			// Use it, leaving over any unused space
			UsedMemorySize += AlignedSize;
			bool bDelete;
			uint32 Padding;
			uint32 Offset = FindEntry->Split(AlignedSize, Alignment, bDelete, Padding, MinAllocationSize);
			if (bDelete)
			{
				FFreeEntry*& PrevRef = Prev ? Prev->Next : FreeList;
				PrevRef = FindEntry->Next;
				delete FindEntry;
			}

			FGrowableAllocationBase* Allocation = CreateAllocationStruct();
			Allocation->Size = AlignedSize;
			Allocation->Padding = Padding;
			Allocation->Offset = Offset;
#if !UE_BUILD_SHIPPING
			Allocation->OwnerType = OwnerType;
#endif
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, GetAddressForTracking(Offset), Size));
			MemoryTrace_Alloc(uint64(GetAddressForTracking(Offset)), Size, Alignment, EMemoryTraceRootHeap::SystemMemory);

			// let the implementation fill in any more
			InitializeAllocationStruct(Allocation);
			return Allocation;
		}

		// if no suitable blocks were found, we must fail
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Failed to allocate GPU memory (Size: %d)"), AlignedSize);
		return nullptr;
	}

	bool Free(FGrowableAllocationBase* Memory)
	{
		// multi-thread protection
		FScopeLock ScopeLock(CriticalSection);

		uint64 Padding = Memory->Padding;
		uint64 Size = Memory->Size;
		uint64 AllocationSize = Padding + Size;
		uint32 Offset = Memory->Offset;

		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, GetAddressForTracking(Offset)));
		MemoryTrace_Free(uint64(GetAddressForTracking(Offset)), EMemoryTraceRootHeap::SystemMemory);

		// we are now done with the Allocation object
		DestroyAllocationStruct(Memory);

		UsedMemorySize -= Size;
		CurrentAllocs--;

		// Search for a place to insert a new free entry
		FFreeEntry* Prev = NULL;
		FFreeEntry* Entry = FreeList;
		while (Entry && Offset > Entry->Location)
		{
			Prev = Entry;
			Entry = Entry->Next;
		}

		// Are we right before this free entry?
		if (Entry && (Offset + Size) == Entry->Location)
		{
			// Join with chunk
			Entry->Location -= uint32(AllocationSize);
			Entry->BlockSize += uint32(AllocationSize);

			// Can we join the two entries?
			if (Prev && (Prev->Location + Prev->BlockSize) == Entry->Location)
			{
				Prev->BlockSize += Entry->BlockSize;
				Prev->Next = Entry->Next;
				MaxFreeEntrySize = FMath::Max(MaxFreeEntrySize, Prev->BlockSize);
				delete Entry;
			}
			else
			{
				MaxFreeEntrySize = FMath::Max(MaxFreeEntrySize, Entry->BlockSize);
			}
			return true;
		}

		// Are we right after the previous free entry?
		if (Prev && (Prev->Location + Prev->BlockSize + Padding) == Offset)
		{
			// Join with chunk
			Prev->BlockSize += uint32(AllocationSize);

			// Can we join the two entries?
			if (Entry && (Prev->Location + Prev->BlockSize) == Entry->Location)
			{
				Prev->BlockSize += Entry->BlockSize;
				Prev->Next = Entry->Next;
				delete Entry;
			}
			MaxFreeEntrySize = FMath::Max(MaxFreeEntrySize, Prev->BlockSize);
			return true;
		}

		// Insert a new entry.
		FFreeEntry* NewFree = new FFreeEntry(Entry, uint32(Offset - Padding), AllocationSize);
		FFreeEntry*& PrevRef = Prev ? Prev->Next : FreeList;
		PrevRef = NewFree;
		MaxFreeEntrySize = FMath::Max(MaxFreeEntrySize, NewFree->BlockSize);
		return true;
	}

	void GetAllocationInfo(uint64& Used, uint64& Free)
	{
		// @todo: Validate this accounts for alignment padding in the right way
		Used = UsedMemorySize;
		Free = HeapSize - UsedMemorySize;
	}

	void ShowFullAllocationList()
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Full Free List:\n"));

		TMap<uint64, uint32> FreeBlockSizeHistogram;

		uint32 Count = 0;
		uint64 Total = 0;

		FFreeEntry* Entry = FreeList;

		while (Entry)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT(" Location: %d Size: %d\n"), Entry->Location, Entry->BlockSize);
			if (FreeBlockSizeHistogram.Contains(Entry->BlockSize))
			{
				FreeBlockSizeHistogram.FindChecked(Entry->BlockSize)++;
			}
			else
			{
				FreeBlockSizeHistogram.Add(Entry->BlockSize) = 1;
			}

			Count++;
			Total += Entry->BlockSize;

			Entry = Entry->Next;
		}

		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Num Blocks: %d Total Size: %d\n"), Count, Total);

		for (auto It = FreeBlockSizeHistogram.CreateIterator(); It; ++It)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT(" %d, %d\n"), It.Key(), It.Value());
		}
	

	}

	// returns an address usable FOR TRACKING ONLY!
	virtual void* GetAddressForTracking(uint32 Offset) = 0;

private:
	class FFreeEntry
	{
	public:
		/** Constructor */
		FFreeEntry(FFreeEntry *NextEntry, uint32 InLocation, uint64 InSize)
			: Location(InLocation)
			, BlockSize(uint32(InSize))
			, Next(NextEntry)
		{
		}

		/**
		* Determine if the given allocation with this alignment and size will fit
		* @param AllocationSize	Already aligned size of an allocation
		* @param Alignment			Alignment of the allocation (location may need to increase to match alignment)
		* @return true if the allocation will fit
		*/
		bool CanFit(uint64 AlignedSize, uint32 Alignment)
		{
			// location of the aligned allocation
			uint32 AlignedLocation = Align(Location, Alignment);

			// if we fit even after moving up the location for alignment, then we are good
			return (AlignedSize + (AlignedLocation - Location)) <= BlockSize;
		}

		/**
		* Take a free chunk, and split it into a used chunk and a free chunk
		* @param UsedSize	The size of the used amount (anything left over becomes free chunk)
		* @param Alignment	The alignment of the allocation (location and size)
		* @param bDelete Whether or not to delete this FreeEntry (ie no more is left over after splitting)
		*
		* @return The location of the free data
		*/
		uint32 Split(uint64 UsedSize, uint32 Alignment, bool &bDelete, uint32& Padding, uint32 MinSize)
		{
			// make sure we are already aligned
			check((UsedSize & (Alignment - 1)) == 0);

			// this is the pointer to the free data
			uint32 FreeLoc = Align(Location, Alignment);

			// Adjust the allocated size for any alignment padding
			Padding = FreeLoc - Location;
			uint64 AllocationSize = UsedSize + uint64(Padding);

			// see if there's enough space left over for a new free chunk (of at least a certain size)
			if (BlockSize - AllocationSize >= MinSize)
			{
				// update this free entry to just point to what's left after using the UsedSize
				Location += uint32(AllocationSize);
				BlockSize -= uint32(AllocationSize);
				bDelete = false;
			}
			// if no more room, then just remove this entry from the list of free items
			else
			{
				bDelete = true;
			}

			// return a usable pointer!
			return FreeLoc;
		}

		/** Offset in the heap */
		uint32		Location;

		/** Size of the free block */
		uint32		BlockSize;

		/** Next block of free memory. */
		FFreeEntry*	Next;
	};

//protected:
// @todo make accessors for TGenericGrowableAllocator to call
public:

	// type of this memory, up to the subclass to define what it means
	uint32 MemoryType;

	/** Size of the heap */
	uint64 HeapSize;

	/** Size of used memory */
	uint64 UsedMemorySize;

	/** Size of the largest free entry (will be MaxFreeEntrySizeDirty if unknown) */
	uint32 MaxFreeEntrySize;

	/** List of free blocks */
	FFreeEntry* FreeList;

	/** Shared critical section */
	FCriticalSection* CriticalSection;

	static const uint32 MaxFreeEntrySizeDirty = 0xffffffff;
};


/**
 * Allocator that will grow as needed with direct mapped memory for a given memory type
 */
template <typename ChunkAllocatorType, typename GrowableAllocationBaseType>
class TGenericGrowableAllocator : public FGrowableMallocBase
{
public:
	
	/**
	 * Constructor
	 * Internally allocates address space for use only by this allocator
	 *
	 * @param Type - The templated memory type to allocate with this allocator
	 * @param StatRegion - The region of memory this is responsible for, for updating the region max sizes
	 */
	TGenericGrowableAllocator(uint64 InitialSize, uint32 InType, uint32 InSubAllocationAlignment, FName InStatRegionName, const FName* InOwnerTypeToStatIdMap, void* InUserData)
		: SubAllocationAlignment(InSubAllocationAlignment)
		, MemoryType(InType)
		, StatRegionName(InStatRegionName)
		, OwnerTypeToStatIdMap(InOwnerTypeToStatIdMap)
		, UserData(InUserData)
	{
		// create initial chunk
		if (InitialSize > 0)
		{
			CreateAllocChunk(InitialSize);
		}
	}

	
	/**
	* Destructor
	*/
	virtual ~TGenericGrowableAllocator()
	{
		// remove any existing chunks
		for (int32 Index = 0; Index < AllocChunks.Num(); Index++)
		{
			ChunkAllocatorType* Chunk = AllocChunks[Index];
			if (Chunk)
			{
#if !UE_BUILD_SHIPPING
				if (!Chunk->IsEmpty())
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Trying to free a non-empty chunk."));
					DumpMemoryInfo();
				}
#endif

				RemoveAllocChunk(Chunk);
			}
		}
	}


	GrowableAllocationBaseType* Malloc(uint32 Size, uint32 Alignment, int32 OwnerType)
	{
		// make sure we have some minimal alignment
		Alignment = FPlatformMath::Max<uint32>(Alignment, SubAllocationAlignment);

		// multi-thread protection
		FScopeLock ScopeLock(&CriticalSection);

		ChunkAllocatorType *AvailableChunk = NULL;

		// align the size to match what Malloc does below
		const uint32 AlignedSize = Align<uint32>(Size, Alignment);

		// Update stats.
		const uint32 WastedSize = AlignedSize - Size;
		TotalAllocated += Size;
		TotalWaste += WastedSize;
		CurrentAllocs++;
		TotalAllocs++;

		// search for an existing alloc chunk with enough space
		for (int32 ChunkIndex = 0; ChunkIndex < AllocChunks.Num(); ChunkIndex++)
		{
			ChunkAllocatorType* Chunk = AllocChunks[ChunkIndex];
			if (Chunk && Chunk->CanFitEntry(AlignedSize, Alignment))
			{
				AvailableChunk = Chunk;
				break;
			}
		}

		// create a new chunk with enough space + alignment to Switch_GrowableHeapAlignment_MB and allocate out of it
		if (AvailableChunk == nullptr)
		{
			AvailableChunk = CreateAllocChunk(AlignedSize);
		}

		// allocate from the space in the chunk
		GrowableAllocationBaseType* Result = nullptr;
		if (AvailableChunk)
		{
			Result = (GrowableAllocationBaseType*)AvailableChunk->Malloc(AlignedSize, Alignment, SubAllocationAlignment, OwnerType);
		}

		if (AvailableChunk == nullptr || Result == nullptr)
		{
			OutOfMemory(AlignedSize);
			return nullptr;
		}

#if ALLOCATION_HISTOGRAM
		TotalAllocationsHistogram.FindOrAdd(AlignedSize)++;
		PeakAllocationsHistogram.FindOrAdd(AlignedSize)++;

		// track per type allocation info
		AllocationInfo& TypeAllocInfo = PerTypeAllocationInfo.FindOrAdd(Result->OwnerType);

		TypeAllocInfo.Counts.Allocations++;
		TypeAllocInfo.TotalAllocated += Result->Size;
#if ALLOCATION_HISTOGRAM_DETAILED
		TypeAllocInfo.AllocationHistogram.FindOrAdd(Result->Size).Allocations++;
#endif
#endif // ALLOCATION_HISTOGRAM 

#if STATS
		if (OwnerTypeToStatIdMap)
		{
			INC_MEMORY_STAT_BY_FName(OwnerTypeToStatIdMap[Result->OwnerType], Result->Size);
		}
#endif

		return Result;
	}

	bool Free(GrowableAllocationBaseType* Memory)
	{
		if (Memory == nullptr)
		{
			return true;
		}

		// multi-thread protection
		FScopeLock ScopeLock(&CriticalSection);

		// starting address space idx used by the chunk containing the allocation
		for (int32 ChunkIndex = 0; ChunkIndex < AllocChunks.Num(); ChunkIndex++)
		{
			ChunkAllocatorType* Chunk = AllocChunks[ChunkIndex];
			if (Chunk && Chunk->DoesChunkContainAllocation(Memory))
			{
#if ALLOCATION_HISTOGRAM
				PeakAllocationsHistogram[Memory->Size]--;

				// untrack per type allocation info
				AllocationInfo& TypeAllocInfo = PerTypeAllocationInfo.FindOrAdd(Memory->OwnerType);

				TypeAllocInfo.TotalAllocated -= Memory->Size;
				TypeAllocInfo.Counts.Frees++;
#if ALLOCATION_HISTOGRAM_DETAILED
				TypeAllocInfo.AllocationHistogram[Memory->Size].Frees++;
#endif 
#endif // ALLOCATION_HISTOGRAM

#if STATS
				if (OwnerTypeToStatIdMap)
				{
					DEC_MEMORY_STAT_BY_FName(OwnerTypeToStatIdMap[Memory->OwnerType], Memory->Size);
				}
#endif
				// free space in the chunk
				Chunk->Free(Memory);
				CurrentAllocs--;

				// never toss the only chunk
				if (Chunk->IsEmpty())// && AllocChunks.Num() > 1)
				{
					// if empty then unmap and decommit physical memory
					RemoveAllocChunk(Chunk);
				}

				// return success
				return true;
			}
		}

		// if we got here, we failed to free the pointer
		UE_LOGF(LogCore, Fatal, "Tried to free invalid pointer");
		return false;
	}


	void GetAllocationInfo(uint32&Chunks, uint64& Used, uint64& Free)
	{
		// multi-thread protection
		FScopeLock ScopeLock(&CriticalSection);

		Chunks = 0;
		// pass off to individual alloc chunks
		for (int32 ChunkIndex = 0; ChunkIndex < AllocChunks.Num(); ChunkIndex++)
		{
			ChunkAllocatorType* Chunk = AllocChunks[ChunkIndex];
			if (Chunk)
			{
				uint64 ChunkUsed = 0;
				uint64 ChunkFree = 0;
				Chunk->GetAllocationInfo(ChunkUsed, ChunkFree);
				Used += ChunkUsed;
				Free += ChunkFree;
				Chunks++;
			}
		}
	}

	bool DoesAllocatorContainAddress(const void* Address)
	{
		// multi-thread protection
		FScopeLock ScopeLock(&CriticalSection);

		// loop through the chunks, query each one to see if they contain the address
		for (int32 ChunkIndex = 0; ChunkIndex < AllocChunks.Num(); ChunkIndex++)
		{
			ChunkAllocatorType* Chunk = AllocChunks[ChunkIndex];
			if (Chunk && Chunk->DoesChunkContainAddress(Address))
			{
				return true;
			}
		}
		return false;
	}

	void ShowAllocationInfo()
	{
 		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("   Allocator has %d chunks\n"), AllocChunks.Num());
 		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("   Allocator average allocation size is %d (%lld over %lld allocs)\n"), (uint32)((float)TotalAllocated / (float)TotalAllocs), TotalAllocated, TotalAllocs);
 		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("   Allocator average waste (on top of allocation) size is %d (%lld over %lld allocs)\n"), (uint32)((float)TotalWaste / (float)TotalAllocs), TotalWaste, TotalAllocs);
	}

	void ShowFullAllocationInfo()
	{
		// multi-thread protection
		FScopeLock ScopeLock(&CriticalSection);

#if ALLOCATION_HISTOGRAM
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Total Allocations Histogram:\n"));
		for (auto It = TotalAllocationsHistogram.CreateIterator(); It; ++It)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT(" %d, %d\n"), It.Key(), It.Value());
		}
#endif // ALLOCATION_HISTOGRAM

		int32 NumChunks = 0;
		// pass off to individual alloc chunks
		for (int32 ChunkIndex = 0; ChunkIndex < AllocChunks.Num(); ChunkIndex++)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("\n-----------------\nChunk %d\n"),ChunkIndex);
			ChunkAllocatorType* Chunk = AllocChunks[ChunkIndex];
			if (Chunk)
			{
				Chunk->ShowFullAllocationList();
			}
		}
	}

	void DumpMemoryInfo()
	{
#if ALLOCATION_HISTOGRAM
		// we use LowLevel here because we are usually dumping this out while 
		for (auto InfoIt = PerTypeAllocationInfo.CreateConstIterator(); InfoIt; ++InfoIt)
		{
			const AllocationInfo& Info = InfoIt.Value();

			if (OwnerTypeToStatIdMap)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("      '%s': Total: %lld Allocs: %d Frees: %d\n"),  *OwnerTypeToStatIdMap[InfoIt.Key()].ToString(), Info.TotalAllocated, Info.Counts.Allocations, Info.Counts.Frees);
			}
			else
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("      'OwnerType %d': %lld Allocs: %d Frees: %d\n"), InfoIt.Key(), Info.TotalAllocated, Info.Counts.Allocations, Info.Counts.Frees);
			}
#if ALLOCATION_HISTOGRAM_DETAILED
			for (auto HistoIt = Info.AllocationHistogram.CreateConstIterator(); HistoIt; ++HistoIt)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("           %d, %lld, %lld\n"), HistoIt.Key(), HistoIt.Value().Allocations, HistoIt.Value().Frees);
			}
#endif
		}
#endif // ALLOCATION_HISTOGRAM
	}

private:

	/** Updates the memory stat max sizes when chunks are added/removed */
	void UpdateMemoryStatMaxSizes()
	{
	}

	/**
	* Create a new allocation chunk to fit the requested size. All chunks are aligned to MIN_CHUNK_ALIGNMENT
	*
	* @param Size - size of chunk
	*/
	ChunkAllocatorType* CreateAllocChunk(uint64 Size)
	{
		ChunkAllocatorType* NewChunk = new ChunkAllocatorType(Size, MemoryType, &CriticalSection, UserData);
		NewChunk->Initialize();
		INC_MEMORY_STAT_BY_FName(StatRegionName, NewChunk->HeapSize);

		// add a new entry to list (reusing any empty slots)
		int32 EmptyEntryIdx = AllocChunks.Find(NULL);
		if (EmptyEntryIdx == INDEX_NONE)
		{
			EmptyEntryIdx = AllocChunks.Add(NewChunk);
		}
		else
		{
			AllocChunks[EmptyEntryIdx] = NewChunk;
		}

#if STATS
		UpdateMemoryStatMaxSizes();
#endif

		return NewChunk;
	}


	/**
	* Removes an existing allocated chunk. Unmaps its memory, decommits physical memory back to OS,
	* flushes address entries associated with it, and deletes it
	*
	* @param Chunk - existing chunk to remove
	*/
	void RemoveAllocChunk(ChunkAllocatorType* Chunk)
	{
		checkSlow(Chunk);

		DEC_MEMORY_STAT_BY_FName(StatRegionName, Chunk->HeapSize);

		// remove entry
		int32 FoundIdx = AllocChunks.Find(Chunk);
		check(AllocChunks.IsValidIndex(FoundIdx));
		AllocChunks[FoundIdx] = NULL;
		Chunk->Destroy();
		delete Chunk;

#if STATS
		UpdateMemoryStatMaxSizes();
#endif
	}


	/** triggered during out of memory failure for this allocator */
	 void OutOfMemory(uint32 Size)
	 {
#if !UE_BUILD_SHIPPING
		 FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FGrowableAllocator: OOM allocating %dbytes %fMB"), Size, static_cast<float>(Size) / 1024.0f / 1024.0f);
		 UE_LOGF(LogCore, Fatal, "FGrowableAllocator: OOM allocating %dbytes %fMB", Size, static_cast<float>(Size) / 1024.0f / 1024.0f);
#endif
	 }

	 // size must be aligned at least to this
	 const uint32 SubAllocationAlignment;

	 /** total currently allocated from OS */
	 uint64 CurSizeAllocated;

	 /** Just stat tracking */
	 uint64 TotalAllocationSize;
	 uint64 NumAllocations;

	 /** The type of memory this allocator allocates from the kernel */
	 uint32 MemoryType;

	 /** The stat memory region to update max size */
	 FName StatRegionName;

	 /** For stats/dumping, we use this to convert OwnerType of an allocation to a printable name*/
	 const FName* OwnerTypeToStatIdMap;

	 /** list of currently allocated chunks */
	 TArray<ChunkAllocatorType*> AllocChunks;

	 // extra data to pass to new Chunks
	 void* UserData;

	 // a critical section used to coordinate all access in this instance of the allocator and it's chunks
	 FCriticalSection CriticalSection;

#if ALLOCATION_HISTOGRAM
	 struct AllocFreeCounts
	 {
		 uint32 Allocations;
		 uint32 Frees;
	 };
	 struct AllocationInfo
	 {
		 uint64 TotalAllocated;
#if ALLOCATION_HISTOGRAM_DETAILED
		 TMap<uint32, AllocFreeCounts> AllocationHistogram;
#endif
		 AllocFreeCounts Counts;
	 };

	 TMap<int32, AllocationInfo> PerTypeAllocationInfo;

	 TMap<uint64, uint32> TotalAllocationsHistogram;
	 TMap<uint64, uint32> OutstandingAllocationsHistogram;
	 TMap<uint64, uint32> PeakAllocationsHistogram;
#endif // ALLOCATION_HISTOGRAM
};

