// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/ReturnFromOpenMode.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PreprocessorHelpers.h"
#include "Hash/CityHash.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Crc.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryLayout.h"
#include "Serialization/StructuredArchive.h"

class FArchive;
class FMemoryImageWriter;
class FMemoryUnfreezeContent;
class FOutputDevice;
class FPointerTableBase;
class FSHA1;
class UObject;
struct FBlake3Hash;
template <typename CharType> class TStringBuilderBase;
template <typename T> struct TCanBulkSerialize;
template <typename T> struct TIsPODType;


/**
 * Enumerates known GUID formats.
 */
enum class EGuidFormats
{
	/**
	 * 32 digits.
	 *
	 * For example: "00000000000000000000000000000000"
	 */
	Digits,

	/**
	 * 32 digits in lowercase
	 *
	 * For example: "0123abc456def789abcd123ef4a5b6c7"
	 */
	 DigitsLower,

	/**
	 * 32 digits separated by hyphens.
	 *
	 * For example: 00000000-0000-0000-0000-000000000000
	 */
	DigitsWithHyphens,

	/**
	 * 32 digits separated by hyphens, in lowercase as described by RFC 4122.
	 *
	 * For example: bd048ce3-358b-46c5-8cee-627c719418f8
	 */
	DigitsWithHyphensLower,

	/**
	 * 32 digits separated by hyphens and enclosed in braces.
	 *
	 * For example: {00000000-0000-0000-0000-000000000000}
	 */
	DigitsWithHyphensInBraces,

	/**
	 * 32 digits separated by hyphens and enclosed in parentheses.
	 *
	 * For example: (00000000-0000-0000-0000-000000000000)
	 */
	DigitsWithHyphensInParentheses,

	/**
	 * Comma-separated hexadecimal values enclosed in braces.
	 *
	 * For example: {0x00000000,0x0000,0x0000,{0x00,0x00,0x00,0x00,0x00,0x00,0x00,0x00}}
	 */
	HexValuesInBraces,

	/**
	 * This format is currently used by the FUniqueObjectGuid class.
	 *
	 * For example: 00000000-00000000-00000000-00000000
	*/
	UniqueObjectGuid,

	/**
	 * Base64 characters with dashes and underscores instead of pluses and slashes (respectively)
	 *
	 * For example: AQsMCQ0PAAUKCgQEBAgADQ
	*/
	Short,

	/**
	 * Base-36 encoded, compatible with case-insensitive OS file systems (such as Windows).
	 *
	 * For example: 1DPF6ARFCM4XH5RMWPU8TGR0J
	 */
	Base36Encoded,
};

/**
 * Implements a globally unique identifier.
 */
struct FGuid
{
public:
	// FGuid can be returned from AutoRTFM::Open() via a copy-construction in the closed.
	static constexpr AutoRTFM::EReturnFromOpenMode AutoRTFMReturnFromOpenMode = AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed;

	/** Default constructor. */
	constexpr FGuid()
		: A(0)
		, B(0)
		, C(0)
		, D(0)
	{ }

	/**
	 * Creates and initializes a new GUID from the specified components.
	 *
	 * @param InA The first component.
	 * @param InB The second component.
	 * @param InC The third component.
	 * @param InD The fourth component.
	 */
	explicit constexpr FGuid(uint32 InA, uint32 InB, uint32 InC, uint32 InD)
		: A(InA), B(InB), C(InC), D(InD)
	{ }

	explicit FGuid(const FString& InGuidStr)
	{
		if (!Parse(InGuidStr, *this))
		{
			Invalidate();
		}
	}

public:

	/**
	 * Compares two GUIDs for equality.
	 *
	 * @param X The first GUID to compare.
	 * @param Y The second GUID to compare.
	 * @return true if the GUIDs are equal, false otherwise.
	 */
	friend bool operator==(const FGuid& X, const FGuid& Y)
	{
		return ((X.A ^ Y.A) | (X.B ^ Y.B) | (X.C ^ Y.C) | (X.D ^ Y.D)) == 0;
	}

	/**
	 * Compares two GUIDs for inequality.
	 *
	 * @param X The first GUID to compare.
	 * @param Y The second GUID to compare.
	 * @return true if the GUIDs are not equal, false otherwise.
	 */
	friend bool operator!=(const FGuid& X, const FGuid& Y)
	{
		return ((X.A ^ Y.A) | (X.B ^ Y.B) | (X.C ^ Y.C) | (X.D ^ Y.D)) != 0;
	}

	/**
	 * Compares two GUIDs.
	 *
	 * @param X The first GUID to compare.
	 * @param Y The second GUID to compare.
	 * @return true if the first GUID is less than the second one.
	 */
	friend bool operator<(const FGuid& X, const FGuid& Y)
	{
		return	((X.A < Y.A) ? true : ((X.A > Y.A) ? false :
				((X.B < Y.B) ? true : ((X.B > Y.B) ? false :
				((X.C < Y.C) ? true : ((X.C > Y.C) ? false :
				((X.D < Y.D) ? true : ((X.D > Y.D) ? false : false)))))))); //-V583
	}

	/**
	 * Provides access to the GUIDs components.
	 *
	 * @param Index The index of the component to return (0...3).
	 * @return The component.
	 */
	uint32& operator[](int32 Index)
	{
		checkSlow(Index >= 0);
		checkSlow(Index < 4);

		switch(Index)
		{
		case 0: return A;
		case 1: return B;
		case 2: return C;
		case 3: return D;
		}

		return A;
	}

	/**
	 * Provides read-only access to the GUIDs components.
	 *
	 * @param Index The index of the component to return (0...3).
	 * @return The component.
	 */
	const uint32& operator[](int32 Index) const
	{
		checkSlow(Index >= 0);
		checkSlow(Index < 4);

		switch(Index)
		{
		case 0: return A;
		case 1: return B;
		case 2: return C;
		case 3: return D;
		}

		return A;
	}

	/**
	 * Serializes a GUID from or into an archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param G The GUID to serialize.
	 */
	CORE_API friend FArchive& operator<<(FArchive& Ar, FGuid& G);

	/**
	 * Serializes a GUID from or into a structured archive slot.
	 *
	 * @param Slot The structured archive slot to serialize from or into
	 * @param G The GUID to serialize.
	 */
	CORE_API friend void operator<<(FStructuredArchive::FSlot Slot, FGuid& G);

	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	/**
	* Guid default string conversion.
	*/
	friend FString LexToString(const FGuid& Value)
	{
		return Value.ToString();
	}

	friend void LexFromString(FGuid& Result, const TCHAR* String)
	{
		FGuid::Parse(String, Result);
	}

	bool Serialize(FStructuredArchive::FSlot Slot)
	{
		Slot << *this;
		return true;
	}

public:

	/**
	 * Exports the GUIDs value to a string.
	 *
	 * @param ValueStr Will hold the string value.
	 * @param DefaultValue The default value.
	 * @param Parent Not used.
	 * @param PortFlags Not used.
	 * @param ExportRootScope Not used.
	 * @return true on success, false otherwise.
	 * @see ImportTextItem
	 */
	CORE_API bool ExportTextItem(FString& ValueStr, FGuid const& DefaultValue, UObject* Parent, int32 PortFlags, class UObject* ExportRootScope) const;

	/**
	 * Imports the GUIDs value from a text buffer.
	 *
	 * @param Buffer The text buffer to import from.
	 * @param PortFlags Not used.
	 * @param Parent Not used.
	 * @param ErrorText The output device for error logging.
	 * @return true on success, false otherwise.
	 * @see ExportTextItem
	 */
	CORE_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);

	/**
	 * Invalidates the GUID.
	 *
	 * @see IsValid
	 */
	void Invalidate()
	{
		A = B = C = D = 0;
	}

	/**
	 * Checks whether this GUID is valid or not.
	 *
	 * A GUID that has all its components set to zero is considered invalid.
	 *
	 * @return true if valid, false otherwise.
	 * @see Invalidate
	 */
	bool IsValid() const
	{
		return ((A | B | C | D) != 0);
	}

	/**
	 * Converts this GUID to its string representation.
	 *
	 * @param Format The string format to use.
	 * @return The string representation.
	 */
	FString ToString(EGuidFormats Format = EGuidFormats::Digits) const
	{
		FString Out;
		AppendString(Out, Format);
		return Out;
	}

	/**
	 * Converts this GUID to its string representation using the specified format.
	 *
	 * @param Format The string format to use.
	 */
	CORE_API void AppendString(FString& Out, EGuidFormats Format = EGuidFormats::Digits) const;

	/**
	 * Appends this GUID to the string builder using the specified format.
	 */
	CORE_API void AppendString(FAnsiStringBuilderBase& Builder, EGuidFormats Format = EGuidFormats::DigitsWithHyphensLower) const;
	CORE_API void AppendString(FUtf8StringBuilderBase& Builder, EGuidFormats Format = EGuidFormats::DigitsWithHyphensLower) const;
	CORE_API void AppendString(FWideStringBuilderBase& Builder, EGuidFormats Format = EGuidFormats::DigitsWithHyphensLower) const;

public:

	/**
	 * Calculates the hash for a GUID.
	 *
	 * @param Guid The GUID to calculate the hash for.
	 * @return The hash.
	 */
	[[nodiscard]] friend uint32 GetTypeHash(const FGuid& Guid)
	{
		return uint32(CityHash64((char*)&Guid, sizeof(FGuid)));
	}

public:

	/**
	 * Returns a new GUID.
	 *
	 * @return A new GUID.
	 */
	static CORE_API FGuid NewGuid();
	/**
	 * Create a guid by hashing the given path; this guid will be deterministic when called in multiple cook processes
	 * and will thus avoid cook indeterminism caused by FGuid::NewGuid. ObjectPath and Seed must be deterministic.
	 */
	static CORE_API FGuid NewDeterministicGuid(FStringView ObjectPath, uint64 Seed = 0);
	/**
	 * Create a guid from a calculated Blake3 Hash
	 */
	static CORE_API FGuid NewGuidFromHash(const FBlake3Hash& Hash);

	/**
	 * Uses the first 16 bytes from HashData to create a Guid. The first 16 bytes of HashData should be premixed to
	 * include all cryptographically necessary bytes from the hash to describe larger dataset, so that changing
	 * any byte of the dataset will change the first 16 bytes of HashData. If HashData is less than 16 bytes, it is
	 * padded with 0 to create the guid.
	 */
	static CORE_API FGuid NewGuidFromHashBytes(const void* HashData, int64 DataLen);

	/**
	 * Returns a GUID which is a combination of the two provided ones.
	 *
	 * @return The combined GUID.
	 */
	static CORE_API FGuid Combine(const FGuid& GuidA, const FGuid& GuidB);

	/**
	 * Returns a GUID which is a combination of the two provided ones, ignoring any invalid ones in the process.
	 *
	 * @return The combined GUID, invalid if both inputs are invalid.
	 */
	static CORE_API FGuid CombineValid(const FGuid& GuidA, const FGuid& GuidB);

	/**
	 * Converts a string to a GUID.
	 *
	 * @param GuidString The string to convert.
	 * @param OutGuid Will contain the parsed GUID.
	 * @return true if the string was converted successfully, false otherwise.
	 * @see ParseExact, ToString
	 */
	static CORE_API bool Parse(const TCHAR* GuidString, FGuid& OutGuid);
	static CORE_API bool Parse(const FString& GuidString, FGuid& OutGuid);
	static CORE_API bool Parse(FStringView GuidString, FGuid& OutGuid);

	/**
	 * Converts a string with the specified format to a GUID.
	 *
	 * @param GuidString The string to convert.
	 * @param Format The string format to parse.
	 * @param OutGuid Will contain the parsed GUID.
	 * @return true if the string was converted successfully, false otherwise.
	 * @see Parse, ToString
	 */
	static CORE_API bool ParseExact(const TCHAR* GuidString, EGuidFormats Format, FGuid& OutGuid);
	static CORE_API bool ParseExact(const FString& GuidString, EGuidFormats Format, FGuid& OutGuid);
	static CORE_API bool ParseExact(FStringView GuidString, EGuidFormats Format, FGuid& OutGuid);

//private:
public:

	/** Holds the first component. */
	uint32 A;

	/** Holds the second component. */
	uint32 B;

	/** Holds the third component. */
	uint32 C;

	/** Holds the fourth component. */
	uint32 D;
};
template<> struct TCanBulkSerialize<FGuid> { enum { Value = true }; };
DECLARE_INTRINSIC_TYPE_LAYOUT(FGuid);

template <> struct TIsPODType<FGuid> { enum { Value = true }; };

template <typename CharType>
inline TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const FGuid& Value)
{
	Value.AppendString(Builder);
	return Builder;
}
