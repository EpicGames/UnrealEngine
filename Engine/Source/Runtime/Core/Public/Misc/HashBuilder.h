// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/GetTypeHashable.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Hash/CityHash.h"

/**
 * Class for computing a hash of multiple types, going through GetTypeHash when the type implements it, and
 * fallbacks to raw data hashing when the type doesn't.
 *
 * Note: this hash builder should be used for transient hashes, as some types implements run-dependent hash
 * computations, such as GetTypeHash(FName).
 */
class FHashBuilder
{
public:
	explicit FHashBuilder(uint32 InHash = 0)
		: Hash(~InHash)
	{}

	inline FHashBuilder& AppendRaw(const void* Data, int64 Num)
	{
		Hash = HashCombineFast(Hash, GetTypeHash(CityHash64(static_cast<const char*>(Data),  static_cast<int32>(Num))));
		return *this;
	}

	template <typename T>
	UE_FORCEINLINE_HINT typename TEnableIf<TIsPODType<T>::Value, FHashBuilder&>::Type AppendRaw(const T& InData)
	{
		return AppendRaw(&InData, sizeof(T));
	}

	template <typename T>
	inline FHashBuilder& Append(const T& InData)
	{
		if constexpr (TModels_V<CGetTypeHashable, T>)
		{
			Hash = HashCombineFast(Hash, GetTypeHash(InData));
			return *this;
		}
		else
		{
			return AppendRaw(InData);
		}
	}

	template <typename T>
	inline FHashBuilder& Append(const TArray<T>& InArray)
	{
		for (const T& Value: InArray)
		{
			Append(Value);
		}
		return *this;
	}

	template <typename T>
	inline FHashBuilder& Append(const TSet<T>& InSet)
	{
		for (const T& Value: InSet)
		{
			Append(Value);
		}
		return *this;
	}

	template <typename T, typename U>
	inline FHashBuilder& Append(const TMap<T, U>& InMap)
	{
		for (const TPair<T, U>& Value: InMap)
		{
			Append(Value.Key);
			Append(Value.Value);
		}
		return *this;
	}

	template <typename T>
	UE_FORCEINLINE_HINT FHashBuilder& operator<<(const T& InData)
	{
		return Append(InData);
	}

	UE_FORCEINLINE_HINT uint32 GetHash() const
	{
		return ~Hash;
	}

private:
	uint32 Hash;
};