// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/ArchiveProxy.h"

class FArchive;

struct FHierarchicalLogArchive : private FArchiveProxy
{
public:
	CORE_API FHierarchicalLogArchive(FArchive& InInnerArchive);

	struct FIndentScope
	{
		explicit FIndentScope(FHierarchicalLogArchive* InAr = nullptr)
			: Ar(InAr)
		{
			if (Ar)
			{
				Ar->Indentation++;
			}
		}

		FIndentScope(const FIndentScope& InOther) = delete;

		explicit FIndentScope(FIndentScope&& InOther)
			: Ar(InOther.Ar)
		{
			InOther.Ar = nullptr;
		}

		~FIndentScope()
		{
			if (Ar)
			{
				check(Ar->Indentation);
				Ar->Indentation--;
			}
		}

		FHierarchicalLogArchive* Ar;
	};

	void Print(const TCHAR* InLine)
	{
		WriteLine(InLine, false);
	}

	[[nodiscard]] FIndentScope PrintIndent(const TCHAR* InLine)
	{
		WriteLine(InLine, true);
		return FIndentScope(this);
	}

	template <typename... Types>
	void Printf(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args)
	{
		WriteLine(FString::Printf(Fmt, Args...), false);
	}

	template <typename... Types>
	[[nodiscard]] FIndentScope PrintfIndent(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args)
	{
		WriteLine(FString::Printf(Fmt, Args...), true);
		return FIndentScope(this);
	}

private:
	CORE_API void WriteLine(const FString& InLine, bool bIndent = false);

	int32 Indentation;
};

#if NO_LOGGING
#define UE_SCOPED_INDENT_LOG_ARCHIVE(Code)
#else
#define UE_SCOPED_INDENT_LOG_ARCHIVE(Code) \
	FHierarchicalLogArchive::FIndentScope BODY_MACRO_COMBINE(Scoped,Indent,_,__LINE__) = Code
#endif
