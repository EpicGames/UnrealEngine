// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Misc/CompressionFlags.h"


#define COMPRESSION_FORMAT_FEATURE_NAME "CompressionFormat"

struct ICompressionFormat : public IModularFeature, public IModuleInterface
{
	virtual FName GetCompressionFormatName() = 0;

	UE_DEPRECATED(5.5, "Switch to 64 bit version for memory sizes")
	virtual bool Compress(void* CompressedBuffer, int32& CompressedSize, const void* UncompressedBuffer, int32 UncompressedSize, int32 CompressionData, ECompressionFlags Flags) { return false; };

	// Default 64 bit implementation wraps legacy 32 bit implementation with size checks - will be removed when deprecated function is removed
	virtual bool Compress(void* CompressedBuffer, int64& CompressedSize, const void* UncompressedBuffer, int64 UncompressedSize, uintptr_t CompressionData, ECompressionFlags Flags)
	{
		if (!IntFitsIn<int32>(UncompressedSize) ||
			!IntFitsIn<int32>(CompressedSize) ||
			!IntFitsIn<int32>(CompressionData))
		{
			return false;
		}
		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		return Compress(CompressedBuffer, (int32&)CompressedSize, UncompressedBuffer, (int32)UncompressedSize, (int32)CompressionData, Flags);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	}

	UE_DEPRECATED(5.5, "Switch to 64 bit version for memory sizes")
	virtual bool Uncompress(void* UncompressedBuffer, int32& UncompressedSize, const void* CompressedBuffer, int32 CompressedSize, int32 CompressionData) {return false;};

	// Default 64 bit implementation wraps legacy 32 bit implementation with size checks - will be removed when deprecated function is removed
	virtual bool Uncompress(void* UncompressedBuffer, int64 UncompressedSize, const void* CompressedBuffer, int64 CompressedSize, uintptr_t CompressionData)
	{
		if (!IntFitsIn<int32>(UncompressedSize) ||
			!IntFitsIn<int32>(CompressedSize) ||
			!IntFitsIn<int32>(CompressionData))
		{
			return false;
		}
		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		return Uncompress(UncompressedBuffer, (int32&)UncompressedSize, CompressedBuffer, (int32)CompressedSize, (int32)CompressionData);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	}

	UE_DEPRECATED(5.5, "Switch to 64 bit version for memory sizes")
	virtual int32 GetCompressedBufferSize(int32 UncompressedSize, int32 CompressionData) {return -1;};

	virtual bool GetCompressedBufferSize(int64& OutBufferSize, int64 UncompressedSize, uintptr_t CompressionData)
	{
		if (!IntFitsIn<int32>(UncompressedSize) ||
			!IntFitsIn<int32>(CompressionData))
		{
			checkf(0, TEXT("%s::GetCompressedBufferSize can't handle 64 bits - format needs to upgrade to the new API."), *GetCompressionFormatName().ToString());
			return false;
		}
		
		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		OutBufferSize = GetCompressedBufferSize((int32)UncompressedSize, (int32)CompressionData);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;
		return true;
	}

	virtual uint32 GetVersion() = 0;
	virtual FString GetDDCKeySuffix() = 0;

	// Returns whether the compression format internally decides whether the decreased size is worth the CPU cost
	// of decompressing it. If the format does this check it should report this by returning a compressed buffer
	// larger than the decompressed buffer - this is usually automatic as the format would presumably then send the
	// data uncompressed wrapped in the format's container, which would increase the size.
	// See FCompression::CompressMemoryIfWorthDecompressing.
	virtual bool DoesOwnWorthDecompressingCheck() = 0;
};
