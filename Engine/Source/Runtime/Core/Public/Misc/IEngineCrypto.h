// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Features/IModularFeatures.h"
#include "Containers/ArrayView.h"

typedef void* FRSAKeyHandle;
constexpr inline FRSAKeyHandle InvalidRSAKeyHandle = nullptr;

struct IEngineCrypto : public IModularFeature
{
	/**
	* Get the name of this modular feature
	*/
	static inline FName GetFeatureName()
	{
		static const FName Name("EngineCryptoFeature");
		return Name;
	}

	/**
	 * Shutdown / cleanup the feature
	 */
	virtual void Shutdown() = 0;

	/** 
	* Create a new RSA key from the given little-endian exponents and modulus
	*/
	virtual FRSAKeyHandle CreateRSAKey(const TArrayView<const uint8> InPublicExponent, const TArrayView<const uint8> InPrivateExponent, const TArrayView<const uint8> InModulus) = 0;

	/**
	* Destroy the given RSA key
	*/
	virtual void DestroyRSAKey(FRSAKeyHandle InKey) = 0;

	/**
	* Get the size of bytes of the given RSA key
	*/
	virtual int32 GetKeySize(FRSAKeyHandle InKey) = 0;

	/** 
	* Get the maximum amount of data that can be encrypted using the given key, taking into account minimum padding requirements
	*/
	virtual int32 GetMaxDataSize(FRSAKeyHandle InKey) = 0;

	/**
	* Encrypt the supplied byte data using the given public key
	*/
	virtual int32 EncryptPublic(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, FRSAKeyHandle InKey) = 0;

	/**
	 * Encrypt the supplied byte data using the given private key
	 */
	virtual int32 EncryptPrivate(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, FRSAKeyHandle InKey) = 0;

	/**
	 * Decrypt the supplied byte data using the given public key
	 */
	virtual int32 DecryptPublic(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, FRSAKeyHandle InKey) = 0;

	/**
	 * Decrypt the supplied byte data using the given private key
	 */
	virtual int32 DecryptPrivate(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, FRSAKeyHandle InKey) = 0;
};