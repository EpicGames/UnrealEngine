// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"

/**
 * A generic interface that represents a Filter of ItemType.
 */
template< typename TItemType >
class IFilter
{
public:
	typedef TItemType ItemType;

	virtual ~IFilter()= default;
	
	/** Convenience function for identifying types of filter in logging, telemetry, etc. */
	virtual FString GetName() const { return FString{}; };

	/** Returns whether the specified Item passes the Filter's restrictions */
	virtual bool PassesFilter( TItemType InItem ) const = 0;

	/** Broadcasts anytime the restrictions of the Filter changes */
	DECLARE_EVENT( IFilter<TItemType>, FChangedEvent );
	virtual FChangedEvent& OnChanged() = 0;
};
