// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/**
 * Interface for controlling an object's start/stop life cycle.
 */
class ILifeCycle
{
public:

	/**
	 * Checks whether the object has been started.
	 *
	 * @return true if started, false otherwise.
	 * @see Start, Stop
	 */
	virtual bool IsStarted() const = 0;

	/**
	 * Starts the object.
	 *
	 * @return true if it was started, false otherwise.
	 * @see IsStarted, Stop
	 */
	virtual bool Start() = 0;

	/**
	 * Stops the object.
	 *
	 * @return true if it was stopped, false otherwise.
	 * @see IsStarted, Start
	 */
	virtual bool Stop() = 0;
};
