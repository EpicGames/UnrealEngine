// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/RefCounting.h"

/** Special flags that can be associated with queued work. */
enum class EQueuedWorkFlags : uint8
{
	None,

	/**
	 * Tells the scheduler if this task is allowed to run during another task's
	 * busy wait. The default should be true for most case but it
	 * is sometime useful to avoid it if this task is going to wait on another one,
	 * and that other task busy waits, this could cause a cycle that could deadlock.
	 * (i.e. T1 -> busywait -> picks T2 that then waits on T1 -> deadlock
	 * In this case, we can decide that T2 should never be picked up by busy waits.
	 */
	 DoNotRunInsideBusyWait = (1 << 0),

	 Count
};

ENUM_CLASS_FLAGS(EQueuedWorkFlags);

/**
* Interface for internal data of queued work objects.
*
* This interface can be used to track some data between the individual function invokations of the FQueuedThreadPool
* Usually it is used to store some internal state to support cancellation without having to look it up from a map.
*/
class IQueuedWorkInternalData : public FRefCountedObject
{
public:
	/**
	* called during retraction, when a task is pulled from being worked on.
	* the return value specifies if the cancellation succeded
	*/
	virtual bool Retract() = 0;

public:

	/**
	* Virtual destructor so that child implementations are guaranteed a chance
	* to clean up any resources they allocated.
	*/
	virtual ~IQueuedWorkInternalData() = default;
};

/**
 * Interface for queued work objects.
 *
 * This interface is a type of runnable object that requires no per thread
 * initialization. It is meant to be used with pools of threads in an
 * abstract way that prevents the pool from needing to know any details
 * about the object being run. This allows queuing of disparate tasks and
 * servicing those tasks with a generic thread pool.
 */
class IQueuedWork
{
public:
	/**
	 * This is where the real thread work is done. All work that is done for
	 * this queued object should be done from within the call to this function.
	 */
	virtual void DoThreadedWork() = 0;

	/**
	 * Tells the queued work that it is being abandoned so that it can do
	 * per object clean up as needed. This will only be called if it is being
	 * abandoned before completion. NOTE: This requires the object to delete
	 * itself using whatever heap it was allocated in.
	 */
	virtual void Abandon() = 0;

	/**
	 * Returns any special work flags.
	 */
	virtual EQueuedWorkFlags GetQueuedWorkFlags() const { return EQueuedWorkFlags::None; }

	/**
	 * Returns an approximation of the peak memory (in bytes) this task could require during it's execution.
	 */
	virtual int64 GetRequiredMemory() const
	{
		return -1 /* Negative value means unknown */;
	}

	/**
	 * Returns text to identify the Work, for debug/log purposes only
	 */
	virtual const TCHAR * GetDebugName() const
	{
		return nullptr;
	}

public:

	/**
	 * Virtual destructor so that child implementations are guaranteed a chance
	 * to clean up any resources they allocated.
	 */
	virtual ~IQueuedWork() = default;

	/**
	* Internal data can be used by the pool
	*/
	using IInternalDataType = TRefCountPtr<IQueuedWorkInternalData>;
	IInternalDataType InternalData;	
};
