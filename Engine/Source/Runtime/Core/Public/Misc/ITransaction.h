// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Misc/Guid.h"
#include "Internationalization/Text.h"
#include "UObject/UObjectHierarchyFwd.h"
#include "Change.h"

// Class for handling undo/redo transactions among objects.
typedef void(*STRUCT_DC)( void* TPtr );						// default construct
typedef void(*STRUCT_AR)( class FArchive& Ar, void* TPtr );	// serialize
typedef void(*STRUCT_DTOR)( void* TPtr );					// destruct


/** Different kinds of actions that can trigger a transaction state change */
enum class ETransactionStateEventType : uint8
{
	/** A transaction has been started. This will be followed by a TransactionCanceled or TransactionFinalized event. */
	TransactionStarted,
	/** A transaction was canceled. */
	TransactionCanceled,
	/** A transaction is about to be finalized. It is still safe to temporarily open new transactions within the scope of this event. */
	PreTransactionFinalized,
	/** A transaction was finalized. */
	TransactionFinalized,
	/** A transaction has been finalized and the internal transaction state has been updated */
	PostTransactionFinalized,

	/** A transaction will be used used in an undo/redo operation. This will be followed by a UndoRedoFinalized event. */
	UndoRedoStarted,
	/** A transaction has been used in an undo/redo operation. */
	UndoRedoFinalized,
};


/**
 * Convenience struct for passing around transaction context.
 */
struct FTransactionContext 
{
	FTransactionContext()
		: TransactionId()
		, OperationId()
		, Title()
		, Context()
		, PrimaryObject(nullptr)
	{
	}

	FTransactionContext(const FGuid& InTransactionId, const FGuid& InOperationId, const FText& InSessionTitle, const TCHAR* InContext, UObject* InPrimaryObject) 
		: TransactionId(InTransactionId)
		, OperationId(InOperationId)
		, Title(InSessionTitle)
		, Context(InContext)
		, PrimaryObject(InPrimaryObject)
	{
	}

	bool IsValid() const
	{
		return TransactionId.IsValid() && OperationId.IsValid();
	}

	/** Unique identifier for the transaction, used to track it during its lifetime */
	FGuid TransactionId;
	/** Unique identifier for the active operation on the transaction (if any) */
	FGuid OperationId;
	/** Descriptive title of the transaction */
	FText Title;
	/** The context that generated the transaction */
	FString Context;
	/** The primary UObject for the transaction (if any). */
	UObject* PrimaryObject;
};


/**
 * Interface for transactions.
 *
 * Transactions are created each time an UObject is modified, for example in the Editor.
 * The data stored inside a transaction object can then be used to provide undo/redo functionality.
 */
class ITransaction
{
public:

	/** BeginOperation should be called when a transaction or undo/redo starts */
	virtual void BeginOperation() = 0;

	/** EndOperation should be called when a transaction is finalized or canceled or undo/redo ends */
	virtual void EndOperation() = 0;

	/** Called when this transaction is completed to finalize the transaction */
	virtual void Finalize() = 0;

	/** Applies the transaction. */
	virtual void Apply() = 0;

	/** Gets the full context for the transaction */
	virtual FTransactionContext GetContext() const = 0;

	/**
	 * Report if a transaction should be put in the undo buffer.
	 * A transaction will be transient if it contains PIE objects or result in a no-op.
	 * If this returns true the transaction won't be put in the transaction buffer.
	 * @returns true if the transaction is transient.
	 */
	virtual bool IsTransient() const = 0;

	/** @returns if this transaction tracks PIE objects */
	virtual bool ContainsPieObjects() const = 0;

	/**
	 * Saves an array to the transaction.
	 *
	 * @param Object The object that owns the array.
	 * @param Array The array to save.
	 * @param Index 
	 * @param Count 
	 * @param Oper
	 * @param ElementSize
	 * @param ElementAlignment
	 * @param Serializer
	 * @param Destructor
	 * @see SaveObject
	 */
	virtual void SaveArray( UObject* Object, class FScriptArray* Array, int32 Index, int32 Count, int32 Oper, int32 ElementSize, uint32 ElementAlignment, STRUCT_DC DefaultConstructor, STRUCT_AR Serializer, STRUCT_DTOR Destructor ) = 0;

	/**
	 * Saves an UObject to the transaction.
	 *
	 * @param Object The object to save.
	 *
	 * @see SaveArray
	 */
	virtual void SaveObject( UObject* Object ) = 0;

	/**
	 * Stores a command that can be used to undo a change to the specified object.  This may be called multiple times in the
	 * same transaction to stack up changes that will be rolled back in reverse order.  No copy of the object itself is stored.
	 *
	 * @param Object The object the undo change will apply to
	 * @param CustomChange The change that can be used to undo the changes to this object.
	 */
	virtual void StoreUndo( UObject* Object, TUniquePtr<FChange> CustomChange ) = 0;

	/**
	 * Sets the transaction's primary object.
	 *
	 * @param Object The primary object to set.
	 */
	virtual void SetPrimaryObject( UObject* Object ) = 0;

	/**
	 * Snapshots a UObject within the transaction.
	 *
	 * @param Object	The object to snapshot.
	 * @param Property	The optional list of properties that have potentially changed on the object (to avoid snapshotting the entire object).
	 */
	virtual void SnapshotObject( UObject* Object, TArrayView<const FProperty*> Properties ) = 0;

	/**
	 * Does the transaction know that the object is being modified.
	 */
	virtual bool ContainsObject(const UObject* Object) const = 0;
};
