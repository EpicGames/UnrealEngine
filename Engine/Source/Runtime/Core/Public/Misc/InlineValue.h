// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Templates/MemoryOps.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/Decay.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/UnrealTemplate.h"

/**
 * A container type that houses an instance of BaseType in inline memory where it is <= MaxInlineSize,
 * or in a separate heap allocation where it's > MaxInlineSize.
 *
 * Can be viewed as a TUniquePtr with a small allocation optimization.
 */
template<typename BaseType, uint16 DesiredMaxInlineSize=64, uint8 DefaultAlignment=8>
class TInlineValue
{
public:
	/**
	 * Default construction to an empty container
	 */
	TInlineValue()
		: bIsValid(false), bInline(true)
	{
	}

	/**
	 * Construction from any type relating to BaseType.
	 */
	template<
		typename T,
		typename = typename TEnableIf<TPointerIsConvertibleFromTo<typename TDecay<T>::Type, BaseType>::Value>::Type
	>
	TInlineValue(T&& In)
		: bIsValid(false)
	{
		InitializeFrom<typename TDecay<T>::Type>(Forward<T>(In));
	}

	/**
	 * In-place construction of BaseType from a set of arguments.
	 */
	template<typename... ArgTypes>
	TInlineValue(EInPlace, ArgTypes&&... Args)
		: bIsValid(false)
	{
		InitializeFrom<BaseType>(Forward<ArgTypes>(Args)...);
	}

	/**
	 * Destructor
	 */
	~TInlineValue()
	{
		Reset();
	}

	/**
	 * Move construction/assignment
	 */
	TInlineValue(TInlineValue&& In)
		: bIsValid(false), bInline(true)
	{
		*this = MoveTemp(In);
	}
	TInlineValue& operator=(TInlineValue&& In)
	{
		Reset(MoveTemp(In));
		return *this;
	}

	/**
	 * Copy construction/assignment is disabled
	 */
	TInlineValue(const TInlineValue& In) = delete;
	TInlineValue& operator=(const TInlineValue& In) = delete;

	/**
	 * Move assignment from any type relating to BaseType.
	 */
	template<typename T>
	typename TEnableIf<TPointerIsConvertibleFromTo<typename TDecay<T>::Type, BaseType>::Value, TInlineValue&>::Type operator=(T&& In)
	{
		*this = TInlineValue(Forward<T>(In));
		return *this;
	}

	/**
	 * Reset this container to wrap a new type
	 */
	void Reset(TInlineValue&& In)
	{
		Reset();

		if (In.bIsValid)
		{
			// Steal the object contained within 'In'
			bIsValid = true;
			In.bIsValid = false;

			bInline = In.bInline;
			Data = In.Data;
		}
	}

	/**
	 * Reset this container back to its empty state
	 */
	void Reset()
	{
		if (bIsValid)
		{
			BaseType& Value = GetValue();
			// Set bIsValid immediately to avoid double-deletion on potential re-entry
			bIsValid = false;
			Value.~BaseType();
			ConditionallyDestroyAllocation();
		}
	}

	/**
	 * Emplace a new type (deriving from BaseType) into this inline value
	 */
	template <typename T, typename... ArgsType>
	inline void Emplace(ArgsType&&... Args)
	{
		static_assert(TPointerIsConvertibleFromTo<T, BaseType>::Value, "T must derive from BaseType.");
		Reset();
		InitializeFrom<T>(Forward<ArgsType>(Args)...);
	}

	/**
	 * Check if this container is wrapping a valid object
	 */
	UE_FORCEINLINE_HINT bool IsValid() const
	{
		return bIsValid;
	}

	/**
	 * Access the wrapped object's base type
	 * @return A reference to the object. Will assert where IsValid() is false.
	 */
	inline BaseType& GetValue() const
	{
		checkf(bIsValid, TEXT("It is an error to call GetValue() on an invalid TInlineValue. Please either check IsValid() or use Get(DefaultValue) instead."));
		return bInline ? (BaseType&)Data : **((BaseType**)&Data);
	}

	/**
	 * Get the wrapped object, or a user-specified default
	 * @return The object, or the user-specified default
	 */
	UE_FORCEINLINE_HINT const BaseType& Get(const BaseType& Default) const
	{
		return bIsValid ? GetValue() : Default;
	}

	/**
	 * Get a pointer the wrapped object, or a user-specified default
	 * @return A pointer to the object, or the user-specified default
	 */
	UE_FORCEINLINE_HINT BaseType* GetPtr(BaseType* Default = nullptr) const
	{
		return bIsValid ? &GetValue() : Default;
	}

	UE_FORCEINLINE_HINT BaseType&		operator*() const	{ return GetValue(); }
	UE_FORCEINLINE_HINT BaseType*		operator->() const	{ return &GetValue(); }

	/**
	 * Reserve space for a structure derived from BaseType, of the size and alignment specified .
	 * @note Does not initialize the memory in anyway
	 */
	void* Reserve(uint32 InSize, uint32 InAlignment)
	{
		Reset();

		bInline = InSize <= MaxInlineSize && InAlignment <= DefaultAlignment;

		ConditionallyAllocateObject(InSize, InAlignment);
		bIsValid = true;
		
		return &GetValue();
	}

private:

	template<typename T, typename... ArgsType>
	void InitializeFrom(ArgsType&&... Args)
	{
		bInline = sizeof(T) <= MaxInlineSize && alignof(T) <= DefaultAlignment;

		// Allocate the object
		ConditionallyAllocateObject(sizeof(T), alignof(T));
		bIsValid = true;

		// Placement new our value into the structure
		::new((void*)&GetValue()) T(Forward<ArgsType>(Args)...);

		checkf((void*)&GetValue() == (BaseType*)((T*)&GetValue()), TEXT("TInlineValue cannot operate with multiple inheritance objects."));
	}

	void ConditionallyAllocateObject(uint32 Size, uint32 Alignment)
	{
		if (!bInline)
		{
			// We store a *ptr* to the data in the aligned bytes, when we allocate on the heap
			BaseType* Allocation = (BaseType*)FMemory::Malloc(Size, Alignment);
			*((BaseType**)&Data) = Allocation;
		}
	}

	void ConditionallyDestroyAllocation()
	{
		if (!bInline)
		{
			FMemory::Free(*((void**)&Data));
		}
	}

private:
	/** ~We cannot allow an allocation of less than the size of a pointer */
	enum { MaxInlineSize = DesiredMaxInlineSize < sizeof(void*) ? sizeof(void*) : DesiredMaxInlineSize };

	/** Type-erased bytes containing either a BaseType& or heap allocated BaseType* */
	TAlignedBytes<MaxInlineSize, DefaultAlignment> Data;

	/** true where this container is wrapping a valid object */
	bool bIsValid : 1;

	/** true where InlineBytes points to a valid BaseType&, false if it's a heap allocated BaseType* */
	bool bInline : 1;
};

/**
 * Construct a new TInlineValue<BaseType> from the specified user type, and arguments
 */
template<typename BaseType, typename UserType, uint8 MaxInlineSize = 64, uint8 DefaultAlignment = 8, typename... ArgsType>
TInlineValue<BaseType, MaxInlineSize, DefaultAlignment> MakeInlineValue(ArgsType... Args)
{
	return TInlineValue<BaseType, MaxInlineSize, DefaultAlignment>(UserType(Forward<ArgsType>(Args)...));
}
