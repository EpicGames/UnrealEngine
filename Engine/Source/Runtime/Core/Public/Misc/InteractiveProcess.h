// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Queue.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "Logging/LogMacros.h"
#include "Misc/DateTime.h"
#include "Misc/Timespan.h"

class FRunnableThread;

DECLARE_LOG_CATEGORY_EXTERN(LogInteractiveProcess, Log, All);

/**
* Declares a delegate that is executed when a interactive process completed.
*
* The first parameter is the process return code.
*/
DECLARE_DELEGATE_TwoParams(FOnInteractiveProcessCompleted, int32, bool);

/**
* Declares a delegate that is executed when a interactive process produces output.
*
* The first parameter is the produced output.
*/
DECLARE_DELEGATE_OneParam(FOnInteractiveProcessOutput, const FString&);

/**
* Implements an external process that can be interacted.
*/
class FInteractiveProcess
	: public FRunnable
{
public:

	/**
	* Creates a new interactive process.
	*
	* @param InURL The URL of the executable to launch.
	* @param InParams The command line parameters.
	* @param InHidden Whether the window of the process should be hidden.
	*/
	CORE_API FInteractiveProcess(const FString& InURL, const FString& InParams, bool InHidden, bool LongTime = false);

	/**
	* Creates a new interactive process.
	*
	* @param InURL The URL of the executable to launch.
	* @param InParams The command line parameters.
	* @param InWorkingDir The URL of the working dir where the executable should launch.
	* @param InHidden Whether the window of the process should be hidden.
	*/
	CORE_API FInteractiveProcess(const FString& InURL, const FString& InParams, const FString& InWorkingDir, bool InHidden, bool LongTime = false);

	/** Destructor. */
	CORE_API ~FInteractiveProcess();

	/**
	* Gets the duration of time that the task has been running.
	*
	* @return Time duration.
	*/
	CORE_API FTimespan GetDuration() const;

	/**
	* Gets the Process Handle. The instance can be invalid if the process was not created.
	*
	* @return The Process Handle
	*/
	FProcHandle GetProcessHandle() const
	{
		return ProcessHandle;
	}

	/**
	* Checks whether the process is still running.
	*
	* @return true if the process is running, false otherwise.
	*/
	bool IsRunning() const
	{
		return Thread != nullptr;
	}

	/**
	* Launches the process
	*
	* @return True if succeed
	*/
	CORE_API bool Launch();

	/**
	* Returns a delegate that is executed when the process has been canceled.
	*
	* @return The delegate.
	*/
	FSimpleDelegate& OnCanceled()
	{
		return CanceledDelegate;
	}

	/**
	* Returns a delegate that is executed when the interactive process completed.
	* Delegate won't be executed if process terminated without user wanting
	*
	* @return The delegate.
	*/
	FOnInteractiveProcessCompleted& OnCompleted()
	{
		return CompletedDelegate;
	}

	/**
	* Returns a delegate that is executed when a interactive process produces output.
	*
	* @return The delegate.
	*/
	FOnInteractiveProcessOutput& OnOutput()
	{
		return OutputDelegate;
	}

	/**
	* Sends the string message when process is ready
	*
	* @param Message to be sent
	*/
	CORE_API void SendWhenReady(const FString &Message);

	/**
	* Sends the data message when process is ready
	*
	* @param Data to be sent
	*/
	CORE_API void SendWhenReady(const TArray<uint8> &Data);

	/**
	* Returns the return code from the exited process
	*
	* @return Process return code
	*/
	int GetReturnCode() const
	{
		return ReturnCode;
	}

	/**
	* Cancels the process.
	*
	* @param InKillTree Whether to kill the entire process tree when canceling this process.
	*/
	void Cancel(bool InKillTree = false)
	{
		bCanceling = true;
		bKillTree = InKillTree;
	}

	// FRunnable interface

	virtual bool Init() override
	{
		return true;
	}

	CORE_API virtual uint32 Run() override;

	virtual void Stop() override
	{
		Cancel();
	}

	virtual void Exit() override { }

protected:

	/**
	* Processes the given output string.
	*
	* @param Output The output string to process.
	*/
	CORE_API void ProcessOutput(const FString& Output);

	/**
	 * Takes the first message to be sent from MessagesToProcess, if there is one, and sends it to process
	 */
	CORE_API void SendMessageToProcessIf();

private:
	// Whether the process is being canceled. */
	bool bCanceling : 1;

	// Whether the window of the process should be hidden. */
	bool bHidden : 1;

	// Whether to kill the entire process tree when cancelling this process. */
	bool bKillTree : 1;
	
	// How many milliseconds should the process sleep */
	float SleepTime;

	// Holds the URL of the executable to launch. */
	FString URL;

	// Holds the command line parameters. */
	FString Params;

	// Holds the URL of the working dir for the process. */
	FString WorkingDir;

	// Holds the handle to the process. */
	FProcHandle ProcessHandle;

	// Holds the read pipe of parent process. */
	void* ReadPipeParent;

	// Holds the write pipe of parent process. */
	void* WritePipeParent;

	// Holds the read pipe of child process. Should not be used except for testing */
	void* ReadPipeChild;
	
	// Holds the write pipe of child process. Should not be used except for testing */
	void* WritePipeChild;

	// Holds the thread object. */
	FRunnableThread* Thread;

	// Holds the name of thread */
	FString ThreadName;

	// Holds the return code. */
	int ReturnCode;

	// Holds the time at which the process started. */
	FDateTime StartTime;

	// Holds the time at which the process ended. */
	FDateTime EndTime;

	// Holds string messages to be written to pipe when ready */
	TQueue<FString> StringMessagesToProcess;

	// Holds data messages to be written to pipe when ready */
	TQueue<TArray<uint8>> DataMessagesToProcess;

	// Holds a delegate that is executed when the process has been canceled. */
	FSimpleDelegate CanceledDelegate;

	// Holds a delegate that is executed when a interactive process completed. */
	FOnInteractiveProcessCompleted CompletedDelegate;

	// Holds a delegate that is executed when a interactive process produces output. */
	FOnInteractiveProcessOutput OutputDelegate;
};
