// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Facilities to allow types to have an intrusive invalid state which can act as
 * TOptional's 'unset' state, saving space.  A class in such a state will only ever be
 * compared against FIntrusiveUnsetOptionalState or destructed.  It will be trivially
 * copied if it's trivially copyable, otherwise its copy constructor and assignment operator
 * won't be called in this state.
 *
 * A class should implement a constructor taking FIntrusiveUnsetOptionalState, and an 
 * equality comparison operator against FIntrusiveUnsetOptionalState, which will put a
 * class instance into the 'unset' state (in the case of the constructor) and allow testing
 * of its unset state.
 *
 * A public constexpr static data member of type bool called bHasIntrusiveUnsetOptionalState
 * should be defined and set to true.  There must also be a public typedef/alias called
 * IntrusiveUnsetOptionalStateType and set to the type itself.  This enables the optimization
 * of this type within TOptional.
 *
 * These functions should be public - regular user code will not be able to call them as
 * they will not be able to construct an FIntrusiveUnsetOptionalState object to pass.
 *
 * Example:
 *
 * struct FMyType
 * {
 *     // This static member should be constexpr, public and equal to true.
 *     static constexpr bool bHasIntrusiveUnsetOptionalState = true;
 *
 *     // This alias should be set to the class type itself.
 *     using IntrusiveUnsetOptionalStateType = FMyType;
 *
 *     explicit FMyType(int32 InIndex)
 *     {
 *         // Validate class invariant.
 *         check(InIndex >= 0);
 *
 *         Index = InIndex;
 *     }
 *
 *     // This constructor will only ever be called by TOptional<FMyType> to enter its
 *     // 'unset' state.
 *     explicit FMyType(FIntrusiveUnsetOptionalState)
 *     {
 *         // Since negative indices are illegal as per the class invariant, we can use -1
 *         // here as TOptional's 'unset' state, which no legal class instance will have.
 *         Index = -1;
 *     }
 *
 *     // This comparison function will only ever be called by TOptional to check if the
 *     // object is in the 'unset' state.  It does not need to be commutative like most
 *     // comparison operators, nor is an operator!= necessary.
 *     bool operator==(FIntrusiveUnsetOptionalState) const
 *     {
 *         return Index == -1;
 *     }
 *
 * private:
 *     // Non-negative indices are part of the class invariant.
 *     int32 Index;
 * };
 */

#include "Misc/OptionalFwd.h"
#include "Concepts/DecaysTo.h"
#include "Concepts/SameAs.h"

namespace UE::Core::Private
{
	struct FOptional;
}

struct FIntrusiveUnsetOptionalState
{
	// Defined in the global namespace for convenience and consistency
	// with TOptional which is also global.

	template <typename>
	friend struct TOptional;

	friend struct UE::Core::Private::FOptional;

private:
	explicit FIntrusiveUnsetOptionalState() = default;
};

template <typename T>
constexpr bool HasIntrusiveUnsetOptionalState()
{
	// Ensure the type has a nested bHasIntrusiveUnsetOptionalState, and as derived types are not guaranteed to have
	// an intrusive state even if the base does, ensure IntrusiveUnsetOptionalStateType matches the type in the optional.
	if constexpr (requires{ { T::bHasIntrusiveUnsetOptionalState } -> UE::CDecaysTo<bool>; requires UE::CSameAs<const typename T::IntrusiveUnsetOptionalStateType, const T>; })
	{
		return T::bHasIntrusiveUnsetOptionalState;
	}
	else
	{
		return false;
	}
}
