// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** Enum that can be used as an explicit result for iterations. E.g. as the return value of visitor functions. */
enum class EIteration
{
	/** The loop should be stopped. */
	Break,

	/** The loop should continue. */
	Continue,
};
