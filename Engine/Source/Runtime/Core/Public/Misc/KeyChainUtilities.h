// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_UNSUPPORTED - Includes JsonSerializer.h which is not in Core module

#include "Base64.h"
#include "CoreDelegates.h"
#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "IEngineCrypto.h"
#include "NamedAESKey.h"
#include "RSA.h"
#include "Serialization/Archive.h"
#include "Serialization/JsonSerializer.h"

struct FKeyChain
{
public:

	FKeyChain() = default;

	FKeyChain(const FKeyChain& Other)
	{
		SetSigningKey(Other.GetSigningKey());
		SetEncryptionKeys(Other.GetEncryptionKeys());

		if (Other.GetPrincipalEncryptionKey())
		{
			SetPrincipalEncryptionKey(GetEncryptionKeys().Find(Other.GetPrincipalEncryptionKey()->Guid));
		}
	}
	
	FKeyChain(FKeyChain&& Other)
	{
		SetSigningKey(Other.GetSigningKey());
		SetEncryptionKeys(MoveTemp(Other.GetEncryptionKeys()));

		if (Other.GetPrincipalEncryptionKey())
		{
			SetPrincipalEncryptionKey(GetEncryptionKeys().Find(Other.GetPrincipalEncryptionKey()->Guid));
		}
		
		Other.SetSigningKey(InvalidRSAKeyHandle);
		Other.SetPrincipalEncryptionKey(nullptr);
		Other.SetEncryptionKeys(TMap<FGuid, FNamedAESKey>());
	}

	FKeyChain& operator=(const FKeyChain& Other)
	{
		SetSigningKey(Other.GetSigningKey());
		SetEncryptionKeys(Other.GetEncryptionKeys());
		
		if (Other.GetPrincipalEncryptionKey())
		{
			SetPrincipalEncryptionKey(GetEncryptionKeys().Find(Other.GetPrincipalEncryptionKey()->Guid));
		}
		else
		{
			SetPrincipalEncryptionKey(nullptr);
		}

		return *this;
	}

	FKeyChain& operator=(FKeyChain&& Other)
	{
		SetSigningKey(Other.GetSigningKey());
		SetEncryptionKeys(MoveTemp(Other.GetEncryptionKeys()));
		
		if (Other.GetPrincipalEncryptionKey())
		{
			SetPrincipalEncryptionKey(GetEncryptionKeys().Find(Other.GetPrincipalEncryptionKey()->Guid));
		}
		else
		{
			SetPrincipalEncryptionKey(nullptr);
		}

		Other.SetSigningKey(InvalidRSAKeyHandle);
		Other.SetPrincipalEncryptionKey(nullptr);
		Other.SetEncryptionKeys(TMap<FGuid, FNamedAESKey>());

		return *this;
	}

	FRSAKeyHandle GetSigningKey() const { return SigningKey; }
	void SetSigningKey(FRSAKeyHandle key) { SigningKey = key; }

	const FNamedAESKey* GetPrincipalEncryptionKey() const { return MasterEncryptionKey; }
	void SetPrincipalEncryptionKey(const FNamedAESKey* key) { MasterEncryptionKey =key; }

	const TMap<FGuid, FNamedAESKey>& GetEncryptionKeys() const { return EncryptionKeys; }
	TMap<FGuid, FNamedAESKey>& GetEncryptionKeys() { return EncryptionKeys; }

	void SetEncryptionKeys(const TMap<FGuid, FNamedAESKey>& keys) { EncryptionKeys = keys; }

private:
	FRSAKeyHandle SigningKey = InvalidRSAKeyHandle;
	TMap<FGuid, FNamedAESKey> EncryptionKeys;
	const FNamedAESKey* MasterEncryptionKey = nullptr;
};


namespace KeyChainUtilities
{
	/** Flags for controlling keychain loading behavior */
	enum class EKeyChainLoadFlags : uint32
	{
		None = 0,
		LoadSigningKeys = 1 << 0,  // Load RSA signing keys (requires engine crypto system)
		Default = LoadSigningKeys  // Default behavior for backward compatibility
	};
	ENUM_CLASS_FLAGS(EKeyChainLoadFlags);

	/** Result from loading keychain */
	struct FKeyChainLoadResult
	{
		bool bSuccess = false;
		FString ErrorMessage;

		static FKeyChainLoadResult Success()
		{
			FKeyChainLoadResult Result;
			Result.bSuccess = true;
			return Result;
		}

		static FKeyChainLoadResult Error(const FString& Message)
		{
			FKeyChainLoadResult Result;
			Result.bSuccess = false;
			Result.ErrorMessage = Message;
			return Result;
		}

		bool IsOk() const { return bSuccess; }
		operator bool() const { return bSuccess; }
	};

	static FRSAKeyHandle ParseRSAKeyFromJson(TSharedPtr<FJsonObject> InObj)
	{
		TSharedPtr<FJsonObject> PublicKey = InObj->GetObjectField(TEXT("PublicKey"));
		TSharedPtr<FJsonObject> PrivateKey = InObj->GetObjectField(TEXT("PrivateKey"));

		FString PublicExponentBase64, PrivateExponentBase64, PublicModulusBase64, PrivateModulusBase64;

		if (PublicKey->TryGetStringField(TEXT("Exponent"), PublicExponentBase64)
			&& PublicKey->TryGetStringField(TEXT("Modulus"), PublicModulusBase64)
			&& PrivateKey->TryGetStringField(TEXT("Exponent"), PrivateExponentBase64)
			&& PrivateKey->TryGetStringField(TEXT("Modulus"), PrivateModulusBase64))
		{
			check(PublicModulusBase64 == PrivateModulusBase64);

			TArray<uint8> PublicExponent, PrivateExponent, Modulus;
			FBase64::Decode(PublicExponentBase64, PublicExponent);
			FBase64::Decode(PrivateExponentBase64, PrivateExponent);
			FBase64::Decode(PublicModulusBase64, Modulus);

			return FRSA::CreateKey(PublicExponent, PrivateExponent, Modulus);
		}
		else
		{
			return nullptr;
		}
	}

	/**
	 * Load keychain from file with options and error reporting.
	 * @param InFilename Path to crypto keys file
	 * @param OutCryptoSettings Output keychain
	 * @param Flags Control what gets loaded (signing keys, etc.)
	 * @return Result indicating success or error message
	 */
	static FKeyChainLoadResult LoadKeyChainFromFileEx(const FString& InFilename, FKeyChain& OutCryptoSettings, EKeyChainLoadFlags Flags = EKeyChainLoadFlags::Default)
	{
		// Open file
		FArchive* File = IFileManager::Get().CreateFileReader(*InFilename);
		if (!File)
		{
			return FKeyChainLoadResult::Error(FString::Printf(TEXT("Crypto keys file '%s' does not exist"), *InFilename));
		}

		// Parse JSON
		TSharedPtr<FJsonObject> RootObject;
		TSharedRef<TJsonReader<UTF8CHAR>> Reader = TJsonReaderFactory<UTF8CHAR>::Create(File);
		if (!FJsonSerializer::Deserialize(Reader, RootObject) || !RootObject.IsValid())
		{
			delete File;
			return FKeyChainLoadResult::Error(FString::Printf(TEXT("Failed to parse JSON from '%s'"), *InFilename));
		}
		delete File;

		const bool bLoadSigningKeys = EnumHasAnyFlags(Flags, EKeyChainLoadFlags::LoadSigningKeys);

		// Load primary encryption key
		const TSharedPtr<FJsonObject>* EncryptionKeyObject;
		if (RootObject->TryGetObjectField(TEXT("EncryptionKey"), EncryptionKeyObject))
		{
			FString EncryptionKeyBase64;
			if ((*EncryptionKeyObject)->TryGetStringField(TEXT("Key"), EncryptionKeyBase64))
			{
				if (EncryptionKeyBase64.Len() > 0)
				{
					TArray<uint8> Key;
					FBase64::Decode(EncryptionKeyBase64, Key);
					checkf(Key.Num() == sizeof(FAES::FAESKey::Key), TEXT("Primary encryption key has invalid size: %d bytes (expected %d) in file '%s'"), Key.Num(), (int32)sizeof(FAES::FAESKey::Key), *InFilename);
					FNamedAESKey NewKey;
					NewKey.Name = TEXT("Default");
					NewKey.Guid = FGuid();
					FMemory::Memcpy(NewKey.Key.Key, &Key[0], sizeof(FAES::FAESKey::Key));
					OutCryptoSettings.GetEncryptionKeys().Add(NewKey.Guid, NewKey);
				}
			}
		}

		// Load RSA signing key (optional, requires engine crypto)
		const TSharedPtr<FJsonObject>* SigningKey = nullptr;
		if (RootObject->TryGetObjectField(TEXT("SigningKey"), SigningKey))
		{
			if (bLoadSigningKeys)
			{
				// Try to parse RSA key - this may fail if engine crypto isn't initialized
				FRSAKeyHandle KeyHandle = ParseRSAKeyFromJson(*SigningKey);
				if (KeyHandle == InvalidRSAKeyHandle)
				{
					return FKeyChainLoadResult::Error(TEXT("Failed to load RSA signing key (engine crypto may not be initialized)"));
				}
				OutCryptoSettings.SetSigningKey(KeyHandle);
			}
			// If signing keys not requested, just skip them silently
		}

		// Load secondary encryption keys
		const TArray<TSharedPtr<FJsonValue>>* SecondaryEncryptionKeyArray = nullptr;
		if (RootObject->TryGetArrayField(TEXT("SecondaryEncryptionKeys"), SecondaryEncryptionKeyArray))
		{
			for (TSharedPtr<FJsonValue> EncryptionKeyValue : *SecondaryEncryptionKeyArray)
			{
				FNamedAESKey NewKey;
				TSharedPtr<FJsonObject> SecondaryEncryptionKeyObject = EncryptionKeyValue->AsObject();
				FGuid::Parse(SecondaryEncryptionKeyObject->GetStringField(TEXT("Guid")), NewKey.Guid);
				NewKey.Name = SecondaryEncryptionKeyObject->GetStringField(TEXT("Name"));
				FString KeyBase64 = SecondaryEncryptionKeyObject->GetStringField(TEXT("Key"));

				TArray<uint8> Key;
				FBase64::Decode(KeyBase64, Key);
				checkf(Key.Num() == sizeof(FAES::FAESKey::Key), TEXT("Secondary encryption key '%s' (%s) has invalid size: %d bytes (expected %d) in file '%s'"), *NewKey.Name, *NewKey.Guid.ToString(), Key.Num(), (int32)sizeof(FAES::FAESKey::Key), *InFilename);
				FMemory::Memcpy(NewKey.Key.Key, &Key[0], sizeof(FAES::FAESKey::Key));

				checkf(!OutCryptoSettings.GetEncryptionKeys().Contains(NewKey.Guid) || OutCryptoSettings.GetEncryptionKeys()[NewKey.Guid].Key == NewKey.Key, TEXT("Duplicate encryption key GUID %s with conflicting key values in file '%s'"), *NewKey.Guid.ToString(), *InFilename);
				OutCryptoSettings.GetEncryptionKeys().Add(NewKey.Guid, NewKey);
			}
		}

		FGuid EncryptionKeyOverrideGuid;
		OutCryptoSettings.SetPrincipalEncryptionKey(OutCryptoSettings.GetEncryptionKeys().Find(EncryptionKeyOverrideGuid));

		return FKeyChainLoadResult::Success();
	}

	/**
	 * Load keychain from file (legacy function with check/crash on error).
	 * For error handling, use LoadKeyChainFromFileEx instead.
	 */
	static void LoadKeyChainFromFile(const FString& InFilename, FKeyChain& OutCryptoSettings)
	{
		// Call new implementation with default flags (maintains backward compatibility)
		FKeyChainLoadResult Result = LoadKeyChainFromFileEx(InFilename, OutCryptoSettings, EKeyChainLoadFlags::Default);

		// Maintain legacy behavior: crash on error
		checkf(Result.IsOk(), TEXT("Failed to load keychain from '%s': %s"), *InFilename, *Result.ErrorMessage);
	}


	static void ApplyEncryptionKeys(const FKeyChain& KeyChain)
	{
		if (KeyChain.GetEncryptionKeys().Contains(FGuid()))
		{
			FAES::FAESKey DefaultKey = KeyChain.GetEncryptionKeys()[FGuid()].Key;
			FCoreDelegates::GetPakEncryptionKeyDelegate().BindLambda([DefaultKey](uint8 OutKey[32]) { FMemory::Memcpy(OutKey, DefaultKey.Key, sizeof(DefaultKey.Key)); });
		}

		for (const TMap<FGuid, FNamedAESKey>::ElementType& Key : KeyChain.GetEncryptionKeys())
		{
			if (Key.Key.IsValid())
			{
				FCoreDelegates::GetRegisterEncryptionKeyMulticastDelegate().Broadcast(Key.Key, Key.Value.Key);
			}
		}
	}
}
