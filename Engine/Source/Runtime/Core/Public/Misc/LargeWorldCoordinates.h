// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Forward declaration of LWC supported core types
#define UE_DECLARE_LWC_TYPE_EX(TYPE, CC, DEFAULT_TYPENAME, COMPONENT_TYPE)				\
namespace UE { namespace Math { template<typename T> struct T##TYPE; } }				\
typedef UE::Math::T##TYPE<float> F##TYPE##CC##f;					/* FVector3f */		\
typedef UE::Math::T##TYPE<double> F##TYPE##CC##d;					/* FVector3d */		\
typedef UE::Math::T##TYPE<COMPONENT_TYPE> DEFAULT_TYPENAME;			/* FVector */		\
namespace ispc { struct DEFAULT_TYPENAME; }							/* ISPC forward declaration */	

// TODO: Need to fix various UE::Geometry name collisions to support this!
//typedef UE::Math::T##TYPE<COMPONENT_TYPE> F##TYPE##CC;				/* FVector3 */


#define UE_DECLARE_LWC_TYPE_3(TYPE, DIM, UE_TYPENAME)			UE_DECLARE_LWC_TYPE_EX(TYPE, DIM, UE_TYPENAME, double)
#define UE_DECLARE_LWC_TYPE_2(TYPE, DIM)						UE_DECLARE_LWC_TYPE_3(TYPE, DIM, F##TYPE)
#define UE_DECLARE_LWC_TYPE_1(TYPE)								UE_DECLARE_LWC_TYPE_2(TYPE,)

// Necessary to convince the MSVC preprocessor to play ball with variadic args - https://stackoverflow.com/questions/5134523/msvc-doesnt-expand-va-args-correctly
#define FORCE_EXPAND(X) X
#define UE_LWC_MACRO_SELECT(PAD1, PAD2, PAD3, PAD4, MACRO, ...)	MACRO
#define UE_DECLARE_LWC_TYPE_SELECT(...)							FORCE_EXPAND(UE_LWC_MACRO_SELECT(__VA_ARGS__, UE_DECLARE_LWC_TYPE_EX, UE_DECLARE_LWC_TYPE_3, UE_DECLARE_LWC_TYPE_2, UE_DECLARE_LWC_TYPE_1 ))

// Args - TYPE, DIMENSION, [UE_TYPENAME], [COMPONENT_TYPE]. e.g. Vector, 3, FVector, double		// LWC_TODO: Remove COMPONENT_TYPE
#define UE_DECLARE_LWC_TYPE(...)								FORCE_EXPAND(UE_DECLARE_LWC_TYPE_SELECT(__VA_ARGS__)(__VA_ARGS__))

// Use to make any narrowing casts searchable in code when it is updated to work with a 64 bit count/range
#define UE_REAL_TO_FLOAT(argument) static_cast<float>(argument)

// Use to make any narrowing casts searchable in code when it is updated to work with a 64 bit count/range clamped to max float.
#define UE_REAL_TO_FLOAT_CLAMPED_MAX(argument) static_cast<float>(FMath::Min(argument, (FVector::FReal)TNumericLimits<float>::Max()))

// Use to make any narrowing casts searchable in code when it is updated to work with a 64 bit count/range clamped to min / max float.
#define UE_REAL_TO_FLOAT_CLAMPED(argument) static_cast<float>(FMath::Clamp(argument, (FVector::FReal)TNumericLimits<float>::Lowest(), (FVector::FReal)TNumericLimits<float>::Max()))

