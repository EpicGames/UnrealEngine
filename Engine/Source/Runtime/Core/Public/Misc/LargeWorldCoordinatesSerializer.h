// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/NameTypes.h"
#include "UObject/ObjectVersion.h"
#include "Serialization/StructuredArchive.h"

// Serialization helper for core variant types only. DO NOT USE!
#define UE_SERIALIZE_VARIANT_FROM_MISMATCHED_TAG(AR_OR_SLOT, ALIAS, TYPE, ALT_TYPE) LWCSerializerPrivate::SerializeFromMismatchedTag<F##ALT_TYPE>(*this, StructTag, AR_OR_SLOT, NAME_##ALIAS, NAME_##TYPE, NAME_##ALT_TYPE)
namespace LWCSerializerPrivate
{

inline bool IsPreLWC(const FArchive& Ar) { return Ar.UEVer() < EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES; } 
inline bool IsPreLWC(const FStructuredArchive::FSlot& Slot) { return IsPreLWC(Slot.GetUnderlyingArchive()); } 

// SerializeFromMismatchedTag helper for core type use only. DO NOT USE!
template<typename FAltType, typename FType, typename FArSlot>
std::enable_if_t<std::is_floating_point_v<typename FType::FReal>, bool>  SerializeFromMismatchedTag(FType& Target, FName StructTag, FArSlot& ArSlot, FName BaseTag, FName ThisTag, FName AltTag)
{
	if(StructTag == ThisTag || (StructTag == BaseTag && (TIsUECoreVariant<FType, double>::Value || IsPreLWC(ArSlot))))
	{
		// Note: relies on Serialize to handle float/double based on archive version.
		return Target.Serialize(ArSlot);
	}
	else if(StructTag == AltTag || StructTag == BaseTag)
	{
		// Convert from alt type
		FAltType AsAlt;										// TODO: Could we derive this from FType?
		const bool bResult = AsAlt.Serialize(ArSlot);
		Target = static_cast<FType>(AsAlt);					// LWC_TODO: Log precision loss warning for TIsUECoreVariant<FType, float>? Could get spammy.
		return bResult;
	}

	return false;
}

// SerializeFromMismatchedTag helper for core type use only. DO NOT USE!
template<typename FAltType, typename FType, typename FArSlot>
std::enable_if_t<std::is_integral_v<typename FType::IntType>, bool> SerializeFromMismatchedTag(FType& Target, FName StructTag, FArSlot& ArSlot, FName BaseTag, FName ThisTag, FName AltTag)
{
	if (StructTag == ThisTag || (StructTag == BaseTag && (TIsUECoreVariant<FType, int32>::Value || TIsUECoreVariant<FType, uint32>::Value)))
	{
		// Note: Unlike float types int retains a default of 32bits, so no conversion is necessary.
		return Target.Serialize(ArSlot);
	}
	else if (StructTag == AltTag || StructTag == BaseTag)
	{
		// Convert from alt type
		FAltType AsAlt;
		const bool bResult = AsAlt.Serialize(ArSlot);
		Target = static_cast<FType>(AsAlt);
		return bResult;
	}

	return false;
}

} // namespace LWCSerializerPrivate
