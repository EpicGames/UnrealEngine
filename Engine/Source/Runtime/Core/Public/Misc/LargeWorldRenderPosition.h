// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/MathFwd.h"
#include "Math/Matrix.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Misc/LargeWorldCoordinates.h"
#include "Math/DoubleFloat.h"

inline constexpr double UE_LWC_RENDER_TILE_SIZE = 2097152.0;

template <typename TScalar>
struct TLargeWorldRenderScalar
{
private:
	using VectorType = UE::Math::TVector<TScalar>;

public:
	UE_FORCEINLINE_HINT static TScalar GetTileSize()
	{
		return static_cast<TScalar>(UE_LWC_RENDER_TILE_SIZE);
	}

	CORE_API static FVector3f GetTileFor(FVector InPosition);

	static TScalar MakeTile(double InValue)
	{
		return static_cast<TScalar>(FMath::FloorToDouble(InValue / GetTileSize() + 0.5));
	}

	static TScalar MakeQuantizedTile(double InValue, double InQuantization)
	{
		return static_cast<TScalar>(FMath::FloorToDouble((InValue / GetTileSize()) * InQuantization + 0.5) / InQuantization);
	}

	static VectorType MakeTile(const FVector& InValue)
	{
		return VectorType(MakeTile(InValue.X), MakeTile(InValue.Y), MakeTile(InValue.Z));
	}

	static VectorType MakeQuantizedTile(const FVector& InValue, double InQuantization)
	{
		return VectorType(MakeQuantizedTile(InValue.X, InQuantization), MakeQuantizedTile(InValue.Y, InQuantization), MakeQuantizedTile(InValue.Z, InQuantization));
	}

	CORE_API static FMatrix44f SafeCastMatrix(const FMatrix& Matrix);
	CORE_API static FMatrix44f MakeToRelativeWorldMatrix(const FVector Origin, const FMatrix& ToWorld);
	CORE_API static FMatrix    MakeToRelativeWorldMatrixDouble(const FVector Origin, const FMatrix& ToWorld);
	CORE_API static FMatrix44f MakeFromRelativeWorldMatrix(const FVector Origin, const FMatrix& FromWorld);
	CORE_API static FMatrix    MakeFromRelativeWorldMatrixDouble(const FVector Origin, const FMatrix& FromWorld);
	CORE_API static FMatrix44f MakeClampedToRelativeWorldMatrix(const FVector Origin, const FMatrix& ToWorld);
	CORE_API static FMatrix    MakeClampedToRelativeWorldMatrixDouble(const FVector Origin, const FMatrix& ToWorld);

	template <typename TResult = float>
	TResult GetTile() const { return static_cast<TResult>(Tile); }
	template <typename TResult = float>
	TResult GetOffset() const { return static_cast<TResult>(Offset); }
	double GetTileOffset() const { return static_cast<double>(Tile) * GetTileSize(); }
	double GetAbsolute() const { return GetTileOffset() + Offset; }

	TLargeWorldRenderScalar() : Tile(static_cast<TScalar>(0.0)), Offset(static_cast<TScalar>(0.0)) {}
	TLargeWorldRenderScalar(TScalar InTile, TScalar InOffset) : Tile(static_cast<TScalar>(InTile)), Offset(static_cast<TScalar>(InOffset)) {}

	template<typename TInputScalar = double>
	explicit TLargeWorldRenderScalar(const TLargeWorldRenderScalar<TInputScalar>& In)
		: TLargeWorldRenderScalar(In.Tile, In.Offset)
	{ }

	TLargeWorldRenderScalar(double InAbsolute)
	{
		// Tiles are centered on the origin
		Tile = MakeTile(InAbsolute);
		Offset = static_cast<TScalar>(InAbsolute - GetTileOffset());
		Validate(InAbsolute);
	}

private:
	CORE_API void Validate(double InAbsolute);

	TScalar Tile;
	TScalar Offset;
};
using FLargeWorldRenderScalar = TLargeWorldRenderScalar<double>;

template<typename TScalar>
struct TLargeWorldRenderPosition
{
private:
	using LWCScalarType = TLargeWorldRenderScalar<TScalar>;
	using VectorType = UE::Math::TVector<TScalar>;

public:
	template<typename TResult = float>
	UE::Math::TVector<TResult> GetTile() const { return UE::Math::TVector<TResult>(static_cast<TResult>(Tile.X), static_cast<TResult>(Tile.Y), static_cast<TResult>(Tile.Z)); }
	
	FVector GetTileOffset() const
	{ 
		LWCScalarType X(Tile.X, Offset.X);
		LWCScalarType Y(Tile.Y, Offset.Y);
		LWCScalarType Z(Tile.Z, Offset.Z);
		return FVector(X.GetTileOffset(), Y.GetTileOffset(), Z.GetTileOffset());
	}

	template<typename TResult = float>
	UE::Math::TVector<TResult> GetOffset() const { return UE::Math::TVector<TResult>(static_cast<TResult>(Offset.X), static_cast<TResult>(Offset.Y), static_cast<TResult>(Offset.Z)); }
	
	FVector GetAbsolute() const
	{
		LWCScalarType X(Tile.X, Offset.X);
		LWCScalarType Y(Tile.Y, Offset.Y);
		LWCScalarType Z(Tile.Z, Offset.Z);
		return FVector(X.GetAbsolute(), Y.GetAbsolute(), Z.GetAbsolute()); 
	}

	template<typename TInputScalar = double>
	explicit TLargeWorldRenderPosition(const UE::Math::TVector<TInputScalar>& InWorldPosition)
	{
		LWCScalarType X(InWorldPosition.X);
		LWCScalarType Y(InWorldPosition.Y);
		LWCScalarType Z(InWorldPosition.Z);
		Tile = VectorType(X.template GetTile<TScalar>(), Y.template GetTile<TScalar>(), Z.template GetTile<TScalar>());
		Offset = VectorType(X.template GetOffset<TScalar>(), Y.template GetOffset<TScalar>(), Z.template GetOffset<TScalar>());
	}

	template<typename TInputScalar = double>
	explicit TLargeWorldRenderPosition(const UE::Math::TVector4<TInputScalar>& InWorldPosition)
		: TLargeWorldRenderPosition(UE::Math::TVector<TInputScalar>(InWorldPosition.X, InWorldPosition.Y, InWorldPosition.Z))
	{ }

	template<typename TInputScalar = double>
	explicit TLargeWorldRenderPosition(const UE::Math::TVector<TInputScalar>& InTilePosition, const UE::Math::TVector<TInputScalar>& InRelativePosition)
	{
		Tile = VectorType(InTilePosition.X, InTilePosition.Y, InTilePosition.Z);
		Offset = VectorType(InRelativePosition.X, InRelativePosition.Y, InRelativePosition.Z);
	}

	template<typename TInputScalar>
	explicit TLargeWorldRenderPosition(const TLargeWorldRenderPosition<TInputScalar>& In)
		: TLargeWorldRenderPosition(In.Tile, In.Offset)
	{ }

	TLargeWorldRenderPosition()
		: Tile(0, 0, 0), Offset(0, 0, 0)
	{ }

private:
	VectorType Tile;
	VectorType Offset;
};
using FLargeWorldRenderPosition = TLargeWorldRenderPosition<double>;

/* See r.PrimitiveHasTileOffsetData */
extern CORE_API int32 GPrimitiveHasTileOffsetData;
