// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if __cplusplus >= 201703L
#include <new> // IWYU pragma: export
#define UE_LAUNDER(x) std::launder(x)
#else
#define UE_LAUNDER(x) (x)
#endif
