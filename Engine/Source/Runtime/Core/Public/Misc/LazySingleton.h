// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AssertionMacros.h"

/** Allows inline friend declaration without forward-declaring TLazySingleton */
class FLazySingleton
{
protected:
	template<class T> static void Construct(void* Place)	{ ::new (Place) T; }
	template<class T> static void Destruct(T* Instance)		{ Instance->~T(); }
};

/**
 * Lazy singleton that can be torn down explicitly
 * 
 * Defining DISABLE_LAZY_SINGLETON_DESTRUCTION stops automatic static destruction
 * and will instead leak singletons that have not been explicitly torn down.
 * This helps prevent shutdown crashes and deadlocks in quick exit scenarios
 * such as ExitProcess() on Windows.
 *
 * T must be default constructible.
 *
 * Example use case:
 *
 * struct FOo
 * 	{
 * 	static FOo& Get();
 * 	static void TearDown();
 * 
 * 	// If default constructor is private
 * 	friend class FLazySingleton;
 * };
 * 
 * // Only include in .cpp and do *not* inline Get() and TearDown()
 * #include "Misc/LazySingleton.h"
 * 
 * FOo& FOo::Get()
 * {
 * 	return TLazySingleton<FOo>::Get();
 * }
 * 
 * void FOo::TearDown()
 * {
 * 	TLazySingleton<FOo>::TearDown();
 * }
 */
template<class T>
class TLazySingleton final : public FLazySingleton
{
public:
	/**
	* Creates singleton once on first call.
	* Thread-safe w.r.t. other Get() calls.
	* Do not call after TearDown(). 
	*/
	static T& Get()
	{
		return GetLazy(Construct<T>).GetValue();
	}

	/** Destroys singleton. No thread must access the singleton during or after this call. */
	static void TearDown()
	{
		return GetLazy(nullptr).Reset();
	}

	/** Get or create singleton unless it's torn down */
	static T* TryGet()
	{
		return GetLazy(Construct<T>).TryGetValue();
	}

private:
	static TLazySingleton& GetLazy(void(*Constructor)(void*))
	{
		static TLazySingleton Singleton(Constructor);
		return Singleton;
	}

	alignas(T) unsigned char Data[sizeof(T)];
	T* Ptr;

	TLazySingleton(void(*Constructor)(void*))
	{
		if (Constructor)
		{
			Constructor(Data);
		}

		Ptr = Constructor ? (T*)Data : nullptr;
	}

#if (!defined(DISABLE_LAZY_SINGLETON_DESTRUCTION) || !DISABLE_LAZY_SINGLETON_DESTRUCTION)
	~TLazySingleton()
	{
		Reset();
	}
#endif

	T* TryGetValue()
	{
		return Ptr;
	}

	T& GetValue()
	{
		check(Ptr);
		return *Ptr;
	}

	void Reset()
	{
		if (Ptr)
		{
			Destruct(Ptr);
			Ptr = nullptr;
		}
	}
};
