// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFile.h"
#include "Misc/DateTime.h"

struct FDateTime;

/**
 * Visitor to gather local files with their timestamps.
 */
class FLocalTimestampDirectoryVisitor
	: public IPlatformFile::FDirectoryVisitor
{
public:

	/** Relative paths to local files and their timestamps. */
	TMap<FString, FDateTime> FileTimes;

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InFileInterface - The platform file interface to use.
	 * @param InDirectoriesToIgnore - The list of directories to ignore.
	 * @param InDirectoriesToNotRecurse - The list of directories to not visit recursively.
	 * @param bInCacheDirectories - Whether to cache the directories.
	 * @param bInMakeLowerCase - Whether to lower case filenames and directories.
	 */
	CORE_API FLocalTimestampDirectoryVisitor( IPlatformFile& InFileInterface, const TArray<FString>& InDirectoriesToIgnore, const TArray<FString>& InDirectoriesToNotRecurse, bool bInCacheDirectories = false, bool bInMakeLowerCase = false );
	CORE_API virtual ~FLocalTimestampDirectoryVisitor();

public:

	// IPlatformFile::FDirectoryVisitor interface

	CORE_API virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory);

private:

	// true if we want directories in this list. */
	bool bCacheDirectories;

	// true if all filenames and directories should be lower cased. */
	bool bMakeLowerCase;

	// Holds a list of directories that we should not traverse into. */
	TArray<FString> DirectoriesToIgnore;

	// Holds a list of directories that we should only go one level into. */
	TArray<FString> DirectoriesToNotRecurse;

	// Holds the file interface to use for any file operations. */
	IPlatformFile& FileInterface;
};
