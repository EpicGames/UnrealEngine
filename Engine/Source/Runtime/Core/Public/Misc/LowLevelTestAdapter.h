// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if !WITH_LOW_LEVEL_TESTS

#include "Containers/UnrealString.h"
#include "Misc/StringBuilder.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include <sstream>
#include <vector>
#endif

/**
* @brief Captures expressions and their evaluated values.
* Internal use only for the low level tests adapter.
*
* @param	InExpressions	Comma-separated list of expressions
* @param	InExperssionsValues	The list of evaluated expressions values
*/
template<typename... ArgTypes>
FString CaptureExpressionsAndValues(const FString& InExpressions, ArgTypes&&... InExpressionsValues)
{
	auto Args = { InExpressionsValues... };
	auto Iter = Args.begin();

	TStringBuilder<64> Result;

	FString RemainingExpressions = InExpressions;
	FString Expression;
	while (RemainingExpressions.Split(TEXT(","), &Expression, &RemainingExpressions))
	{
		if (Iter == Args.end())
		{
			break;
		}
		else if (Iter != Args.begin())
		{
			Result << TEXT(", ");
		}
		Result << Expression.TrimStartAndEnd() << TEXT(" = ")  << *Iter++;
	}

	if (Iter != Args.end() && !RemainingExpressions.TrimStartAndEnd().IsEmpty())
	{
		Result << TEXT(", ") << RemainingExpressions << TEXT(" = ") << *Iter;
	}

	Result << TEXT("\n");
	return FString(Result.ToString());
}


#define IMPLEMENT_SIMPLE_AUTOMATION_TEST_PRIVATE_LLT( TClass, PrettyName, TFlagsOrTags, FileName, LineNumber ) \
		class TClass : public FAutomationTestBase \
		{ \
		public:\
			TClass( const FString& InName) \
			: FAutomationTestBase( InName, false ) \
			{ \
				TestFlags = ExtractAutomationTestFlags(TFlagsOrTags); \
				PrettyNameDotNotation = FString(PrettyName).Replace(TEXT("::"), TEXT(".")); \
				if (!(TestFlags & EAutomationTestFlags_ApplicationContextMask)) \
				{ \
					TestFlags |= EAutomationTestFlags_ApplicationContextMask; \
				} \
				if (!(TestFlags & EAutomationTestFlags_FilterMask)) \
				{ \
					TestFlags |= EAutomationTestFlags::EngineFilter; \
				} \
				FAutomationTestFramework::Get().RegisterAutomationTestTags(GetBeautifiedTestName(), TFlagsOrTags); \
			} \
			virtual EAutomationTestFlags GetTestFlags() const override { return TestFlags; } \
			virtual bool IsStressTest() const { return false; } \
			virtual uint32 GetRequiredDeviceNum() const override { return 1; } \
			virtual FString GetTestSourceFileName() const override { return FileName; } \
			virtual int32 GetTestSourceFileLine() const override { return LineNumber; } \
		protected: \
			virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const override \
			{ \
				OutBeautifiedNames.Add(PrettyNameDotNotation); \
				OutTestCommands.Add(FString());\
			} \
			void TestBody(const FString& Parameters); \
			virtual bool RunTest(const FString& Parameters) { \
				TestBody(Parameters); \
				return !HasAnyErrors(); \
			} \
			virtual FString GetBeautifiedTestName() const override { return PrettyNameDotNotation; } \
		private:\
			EAutomationTestFlags TestFlags; \
			FString PrettyNameDotNotation; \
		};

#define LLT_JOIN(Prefix, Counter) LLT_JOIN_INNER(Prefix, Counter)
#define LLT_JOIN_INNER(Prefix, Counter) Prefix##Counter

#define TEST_CASE_NAMED_STR(TClass, StrName, PrettyName, TFlagsOrTags) \
		IMPLEMENT_SIMPLE_AUTOMATION_TEST_PRIVATE_LLT(TClass, PrettyName, TFlagsOrTags, __FILE__, __LINE__) \
		namespace \
		{ \
			TClass LLT_JOIN(TClass, Instance)(TEXT(StrName)); \
		} \
		void TClass::TestBody(const FString& Parameters)


#define TEST_CASE_GENERATED_NAME_UNIQUE LLT_JOIN(FLLTAdaptedTest, __COUNTER__)
#define LLT_STR(Macro) #Macro
#define LLT_STR_EXPAND(Macro) LLT_STR(Macro)
#define TEST_CASE_GENERATED_NAME_UNIQUE_STR "LLTAdaptedTest_" LLT_STR_EXPAND(__FILE__) "_" LLT_STR_EXPAND(__LINE__)
// Note: TEST_CASE uses unique names which only work when used inside unique namespace in the same compilation unit?// Use TEST_CASE_NAMED instead and provide an unique global instance name
#define TEST_CASE(PrettyName, TFlagsOrTags) TEST_CASE_NAMED_STR(TEST_CASE_GENERATED_NAME_UNIQUE, TEST_CASE_GENERATED_NAME_UNIQUE_STR, PrettyName, TFlagsOrTags)
#define TEST_CASE_NAMED(ClassName, PrettyName, TFlagsOrTags) TEST_CASE_NAMED_STR(ClassName, #ClassName, PrettyName, TFlagsOrTags)

// Both python and oodle don't trust __LINE__ for unique names, and use __COUNTER__ where possible
#ifdef __COUNTER__
#define MAKE_UNIQUE_IDENT(str) LLT_JOIN(str, __COUNTER__)
#else
#define MAKE_UNIQUE_IDENT(str) LLT_JOIN(str, __LINE__)
#endif

// DISABLED_ makes a unique name for either a function or a lambda such that the linker should strip them.
#define DISABLED_TEST_CASE(...)						static void MAKE_UNIQUE_IDENT(disabled_test_()
#define DISABLED_TEST_CASE_NAMED(ClassName, ...)	static void MAKE_UNIQUE_IDENT(disabled_test_)()
#define DISABLED_SCENARIO(...)						static void MAKE_UNIQUE_IDENT(disabled_scenario_)()
#define DISABLED_SECTION(...)						auto MAKE_UNIQUE_IDENT(disabled_section_) = []()

//-V:CHECK:571,501,547
#define CHECK(...) if (!(__VA_ARGS__)) { FAutomationTestFramework::Get().GetCurrentTest()->AddError(TEXT("Condition failed")); }
//-V:CHECK_FALSE:571,501,547
#define CHECK_FALSE(...) if (!!(__VA_ARGS__)) { FAutomationTestFramework::Get().GetCurrentTest()->AddError(TEXT("Condition expected to return false but returned true")); }
#define CHECKED_IF(...) if (!!(__VA_ARGS__))
#define CHECKED_ELSE(...) if (!(__VA_ARGS__))
//-V:CHECK_MESSAGE:571,501,547
#define CHECK_MESSAGE(Message, ...) if (!(__VA_ARGS__)) { FAutomationTestFramework::Get().GetCurrentTest()->AddError(Message); }
//-V:CHECK_FALSE_MESSAGE:571,501,547
#define CHECK_FALSE_MESSAGE(Message, ...) if (!!(__VA_ARGS__)) { FAutomationTestFramework::Get().GetCurrentTest()->AddError(Message); }
//-V:REQUIRE:571,501,547
#define REQUIRE(...) if (!(__VA_ARGS__)) { FAutomationTestFramework::Get().GetCurrentTest()->AddError(TEXT("Required condition failed, interrupting test")); return; }
//-V:REQUIRE_MESSAGE:571,501,547
#define REQUIRE_MESSAGE(Message, ...) if (!(__VA_ARGS__)) { FAutomationTestFramework::Get().GetCurrentTest()->AddError(Message); return; }
#define STATIC_REQUIRE(...) static_assert(__VA_ARGS__, #__VA_ARGS__);
#define STATIC_CHECK(...) static_assert(__VA_ARGS__, #__VA_ARGS__);
#define STATIC_CHECK_FALSE(...) static_assert(!(__VA_ARGS__), "!(" #__VA_ARGS__ ")");

#define CHECK_EQUALS(What, X, Y) FAutomationTestFramework::Get().GetCurrentTest()->TestEqual(What, X, Y);
#define CHECK_EQUALS_SENSITIVE(What, X, Y) FAutomationTestFramework::Get().GetCurrentTest()->TestEqualSensitive(What, X, Y);
#define CHECK_NOT_EQUALS(What, X, Y) FAutomationTestFramework::Get().GetCurrentTest()->TestNotEqual(What, X, Y);
#define CHECK_NOT_EQUALS_SENSITIVE(What, X, Y) FAutomationTestFramework::Get().GetCurrentTest()->TestNotEqualSensitive(What, X, Y);

#define SECTION(Text) FAutomationTestFramework::Get().GetCurrentTest()->AddInfo(TEXT(Text));
#define FAIL_CHECK(Message) FAutomationTestFramework::Get().GetCurrentTest()->AddError(Message);

#define CAPTURE(...) FAutomationTestFramework::Get().GetCurrentTest()->AddInfo(CaptureExpressionsAndValues(#__VA_ARGS__, __VA_ARGS__));
#define INFO(Message) FAutomationTestFramework::Get().GetCurrentTest()->AddInfo(Message);
#define WARN(Message) FAutomationTestFramework::Get().GetCurrentTest()->AddWarning(Message); 
#define ADD_WARNING(Message) FAutomationTestFramework::Get().GetCurrentTest()->AddWarning(Message); 
#define ADD_ERROR(Message) FAutomationTestFramework::Get().GetCurrentTest()->AddError(Message); 
#define FAIL_ON_MESSAGE(Message) FAutomationTestFramework::Get().GetCurrentTest()->AddExpectedError(Message);


#define SKIP(Message)

#endif // !WITH_LOW_LEVEL_TESTS
