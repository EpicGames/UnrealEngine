// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"

#define ENABLE_MT_DETECTOR DO_CHECK

#if ENABLE_MT_DETECTOR

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "HAL/PlatformTLS.h"
#include "HAL/PreprocessorHelpers.h"
#include "HAL/PlatformStackWalk.h"
#include "Misc/Build.h"
#include "AutoRTFM.h"
#include <atomic>

struct FRWAccessDetector;
class FMRSWRecursiveAccessDetector;

namespace UE::Private
{
	class FMTAccessDetectorSuppressCheckFailuresForUnitTests;

	class FMTAccessDetectorOptions
	{
	private:
		/**
		 * For testing purposes, we can suppress check-failures when access violations are detected. 
		 * Instead, the relevant access methods will just return false.
		 */
		static CORE_API bool bSuppressCheckFailure;

		friend struct ::FRWAccessDetector;
		friend class ::FMRSWRecursiveAccessDetector;
		friend class ::UE::Private::FMTAccessDetectorSuppressCheckFailuresForUnitTests;
	};
}

/**
 * Read write multithread access detector, will check on concurrent write/write and read/write access, but will not on concurrent read access.
 * Note this detector is not re-entrant, see FRWRecursiveAccessDetector and FRWFullyRecursiveAccessDetector. 
 * But FRWAccessDetector should be the default one to start with.
 */
struct FRWAccessDetector
{
public:

	FRWAccessDetector()
		: AtomicValue(0)
		{}

	~FRWAccessDetector()
	{
		checkf(AtomicValue.load(std::memory_order_relaxed) == 0, TEXT("Detector cannot be destroyed while other threads access it"));
	}

	FRWAccessDetector(FRWAccessDetector&& Other)
	{
		checkf(Other.AtomicValue.load(std::memory_order_relaxed) == 0, TEXT("Detector cannot be \"moved out\" while other threads access it"));
	}

	FRWAccessDetector& operator=(FRWAccessDetector&& Other)
	{
		checkf(AtomicValue.load(std::memory_order_relaxed) == 0, TEXT("Detector cannot be modified while other threads access it"));
		checkf(Other.AtomicValue.load(std::memory_order_relaxed) == 0, TEXT("Detector cannot be \"moved out\" while other threads access it"));
		return *this;
	}

	FRWAccessDetector(const FRWAccessDetector& Other)
	{
		checkf(Other.AtomicValue.load(std::memory_order_relaxed) == 0, TEXT("Detector cannot be copied while other threads access it"));
	}

	FRWAccessDetector& operator=(const FRWAccessDetector& Other)
	{
		checkf(AtomicValue.load(std::memory_order_relaxed) == 0, TEXT("Detector cannot be modified while other threads access it"));
		checkf(Other.AtomicValue.load(std::memory_order_relaxed) == 0, TEXT("Detector cannot be copied while other threads access it"));
		return *this;
	}

	/**
	 * Acquires read access, will check if there are any writers
	 * @return true if no errors were detected
	 */
	inline bool AcquireReadAccess() const
	{
		const bool ErrorDetected = (AtomicValue.fetch_add(1, std::memory_order_relaxed) & WriterBits) != 0;
		checkf(!ErrorDetected || UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure, TEXT("Acquiring a read access while there is already a write access"));
		return !ErrorDetected;
	}

	/**
	 * Releases read access, will check if there are any writers
	 * @return true if no errors were detected
	 */
	inline bool ReleaseReadAccess() const
	{
		const bool ErrorDetected = (AtomicValue.fetch_sub(1, std::memory_order_relaxed) & WriterBits) != 0;
		checkf(!ErrorDetected || UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure, TEXT("Another thread asked to have a write access during this read access"));
		return !ErrorDetected;
	}

	/** 
	 * Acquires write access, will check if there are readers or other writers
	 * @return true if no errors were detected
	 */
	inline bool AcquireWriteAccess() const
	{
		const bool ErrorDetected = AtomicValue.fetch_add(WriterIncrementValue, std::memory_order_relaxed) != 0;
		checkf(!ErrorDetected || UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure, TEXT("Acquiring a write access while there are ongoing read or write access"));
		return !ErrorDetected;
	}

	/** 
	 * Releases write access, will check if there are readers or other writers
	 * @return true if no errors were detected
	 */
	inline bool ReleaseWriteAccess() const
	{
		const bool ErrorDetected = AtomicValue.fetch_sub(WriterIncrementValue, std::memory_order_relaxed) != WriterIncrementValue;
		checkf(!ErrorDetected || UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure, TEXT("Another thread asked to have a read or write access during this write access"));
		return !ErrorDetected;
	}

	/**
	 * Upgrades a previously taken `AcquireReadAccess` to `AcquireWriteAccess`, and will check if there are other readers or writers
	 * @return true if no errors were detected
	 */
	inline bool UpgradeReadAccessToWriteAccess() const
	{
		const bool bErrorDetected = AtomicValue.fetch_add(WriterIncrementValue, std::memory_order_relaxed) != 1;

		if (UNLIKELY(bErrorDetected))
		{
			checkf(UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure, TEXT("Upgrading a write access while there are ongoing read or write access"));
			return false;
		}

		// Since we now hold the write lock, we can remove the read lock access we've upgraded to write lock access without re-checking!
		AtomicValue -= 1;
		return true;
	}

protected:

	// We need to do an atomic operation to know there are multiple writers, this is why we reserve more than one bit for them.
	// While firing the check upon acquire write access, the other writer thread could continue and hopefully fire a check upon releasing access so we get both faulty callstacks.
	static constexpr uint32 WriterBits = 0xfff00000;
	static constexpr uint32 WriterIncrementValue = 0x100000;

	mutable std::atomic<uint32> AtomicValue;
};

/**
 * Same as FRWAccessDetector, but support re-entrance on the write access
 * See FRWFullyRecursiveAccessDetector for read access re-entrance when holding a write access
 */
struct FRWRecursiveAccessDetector : public FRWAccessDetector
{
public:
	/**
	 * Acquires write access, will check if there are readers or other writers
	 * @return true if no errors were detected
	 */
	inline bool AcquireWriteAccess() const
	{
		uint32 CurThreadID = FPlatformTLS::GetCurrentThreadId();

		if (WriterThreadID == CurThreadID)
		{
			RecursiveDepth++;
			return true;
		}
		else if (FRWAccessDetector::AcquireWriteAccess())
		{
			check(RecursiveDepth == 0);
			WriterThreadID = CurThreadID;
			RecursiveDepth++;
			return true;
		}
		return false;
	}

	/**
	 * Releases write access, will check if there are readers or other writers
	 * @return true if no errors were detected
	 */
	inline bool ReleaseWriteAccess() const
	{
		uint32 CurThreadID = FPlatformTLS::GetCurrentThreadId();
		if (WriterThreadID == CurThreadID)
		{
			check(RecursiveDepth > 0);
			RecursiveDepth--;

			if (RecursiveDepth == 0)
			{
				WriterThreadID = (uint32)-1;
				return FRWAccessDetector::ReleaseWriteAccess();
			}
			return true;
		}
		else
		{
			// This can happen when a user continues pass a reported error, 
			// just trying to keep things going as best as possible.
			return FRWAccessDetector::ReleaseWriteAccess();
		}
	}

protected:
	mutable uint32 WriterThreadID = (uint32)-1;
	mutable int32 RecursiveDepth = 0;
};

/**
 * Same as FRWRecursiveAccessDetector, but support re-entrance on read access when holding a write access.
 */
struct FRWFullyRecursiveAccessDetector : public FRWRecursiveAccessDetector
{
public:
	/**
	 * Acquires read access, will check if there are any writers
	 * @return true if no errors were detected
	 */
	inline bool AcquireReadAccess() const
	{
		uint32 CurThreadID = FPlatformTLS::GetCurrentThreadId();
		if (WriterThreadID == CurThreadID)
		{
			return true;
		}
		return FRWAccessDetector::AcquireReadAccess();
	}

	/**
	 * Releases read access, will check if there are any writers
	 * @return true if no errors were detected
	 */
	inline bool ReleaseReadAccess() const
	{
		uint32 CurThreadID = FPlatformTLS::GetCurrentThreadId();
		if (WriterThreadID == CurThreadID)
		{
			return true;
		}
		return FRWAccessDetector::ReleaseReadAccess();
	}
};

struct FBaseScopedAccessDetector
{
};

template<typename RWAccessDetector>
struct TScopedReaderAccessDetector : public FBaseScopedAccessDetector
{
public:

	inline TScopedReaderAccessDetector(RWAccessDetector& InAccessDetector)
	: AccessDetector(InAccessDetector)
	{
		AccessDetector.AcquireReadAccess();
	}

	UE_FORCEINLINE_HINT ~TScopedReaderAccessDetector()
	{
		AccessDetector.ReleaseReadAccess();
	}
private:
	RWAccessDetector& AccessDetector;
};

template<typename RWAccessDetector>
UE_FORCEINLINE_HINT TScopedReaderAccessDetector<RWAccessDetector> MakeScopedReaderAccessDetector(RWAccessDetector& InAccessDetector)
{
	return TScopedReaderAccessDetector<RWAccessDetector>(InAccessDetector);
}

template<typename RWAccessDetector>
struct TScopedWriterDetector : public FBaseScopedAccessDetector
{
public:
	inline TScopedWriterDetector(RWAccessDetector& InAccessDetector)
		: AccessDetector(InAccessDetector)
	{
		AccessDetector.AcquireWriteAccess();
	}

	UE_FORCEINLINE_HINT ~TScopedWriterDetector()
	{
		AccessDetector.ReleaseWriteAccess();
	}
private:
	RWAccessDetector& AccessDetector;
};

template<typename RWAccessDetector>
UE_FORCEINLINE_HINT TScopedWriterDetector<RWAccessDetector> MakeScopedWriterAccessDetector(RWAccessDetector& InAccessDetector)
{
	return TScopedWriterDetector<RWAccessDetector>(InAccessDetector);
}

/** 
 * race detector supporting multiple readers (MR) single writer (SW) recursive access, a write from inside a read, a read from inside a write 
 * and all other combinations. is zero initializable. supports destruction while being "accessed"
 */
class FMRSWRecursiveAccessDetector
{
private:
	// despite 0 is a valid TID on some platforms, we store actual TID + 1 to avoid collisions. it's required to use 0 as 
	// an invalid TID for zero-initilization
	static constexpr uint32 InvalidThreadId = 0;

	union FState
	{
		uint64 Value;

		struct
		{
			uint32 ReaderNum : 20;
			uint32 WriterNum : 12;
			uint32 WriterThreadId;
		};

		constexpr FState()
			: Value(0)
		{
			static_assert(sizeof(FState) == sizeof(uint64)); // `FState` is stored in `std::atomic<uint64>`
		}

		explicit constexpr FState(uint64 InValue)
			: Value(InValue)
		{
		}

		constexpr FState(uint32 InReaderNum, uint32 InWriterNum, uint32 InWriterThreadId)
			: ReaderNum(InReaderNum)
			, WriterNum(InWriterNum)
			, WriterThreadId(InWriterThreadId)
		{
		}

		uint32 GetWriterThreadId() const
		{
			return WriterThreadId - 1;
		}

		void SetWriterThreadId(uint32 ThreadId)
		{
			WriterThreadId = ThreadId + 1; // to shift from 0 TID that is considered "invalid"
		}
	};

	// all atomic ops are relaxed to preserve the original memory order, as the detector is compiled out in non-dev builds
	mutable std::atomic<uint64> State{ 0 }; // it's actually FState, but we need to do `atomic::fetch_add` on it

	UE_FORCEINLINE_HINT FState LoadState() const
	{
		return FState(State.load(std::memory_order_relaxed));
	}

	UE_FORCEINLINE_HINT FState ExchangeState(FState NewState)
	{
		return FState(State.exchange(NewState.Value, std::memory_order_relaxed));
	}

	inline FState IncrementReaderNum() const
	{
		constexpr FState OneReader{ 1, 0, 0 };
		return FState(State.fetch_add(OneReader.Value, std::memory_order_relaxed));
	}

	inline FState DecrementReaderNum() const
	{
		constexpr FState OneReader{ 1, 0, 0 };
		return FState(State.fetch_sub(OneReader.Value, std::memory_order_relaxed));
	}

	UE_FORCEINLINE_HINT friend bool operator==(FState L, FState R)
	{
		return L.Value == R.Value;
	}

	inline static bool CheckOtherThreadWriters(FState InState)
	{
		if (InState.WriterNum != 0)
		{
			uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
			if (InState.GetWriterThreadId() != CurrentThreadId)
			{
				ensureMsgf(UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure, 
					TEXT("Data race detected! Acquiring read access on thread %u concurrently with %d writers on thread %u:\nCurrent thread %u callstack:\n%s\nWriter thread %u callstack:\n%s"),
					CurrentThreadId, InState.WriterNum, InState.GetWriterThreadId(), 
					CurrentThreadId, *GetCurrentThreadCallstack(),
					InState.GetWriterThreadId(), *GetThreadCallstack(InState.GetWriterThreadId()));

				return false;
			}
		}
		return true;
	}

	static FString GetCurrentThreadCallstack()
	{
		const SIZE_T StackTraceSize = 65536;
		ANSICHAR StackTrace[StackTraceSize] = { 0 };
		FPlatformStackWalk::StackWalkAndDump(StackTrace, StackTraceSize, 0); 
		return ANSI_TO_TCHAR(StackTrace);
	}

	static FString GetThreadCallstack(uint32 ThreadId)
	{
		if (ThreadId == uint32(-1))
		{
			return TEXT("[none]");
		}

		// to get the callstack of another thread we suspend it first. it's a bad idea to suspend the current thread
		if (FPlatformTLS::GetCurrentThreadId() == ThreadId)
		{
			return GetCurrentThreadCallstack();
		}

		const SIZE_T StackTraceSize = 65536;
		ANSICHAR StackTrace[StackTraceSize] = { 0 };
		FPlatformStackWalk::ThreadStackWalkAndDump(StackTrace, StackTraceSize, 0, ThreadId); 
		return ANSI_TO_TCHAR(StackTrace);
	}

//////////////////////////////////////////////////////////////////////
// DestructionSentinel
// This access detector supports its destruction (along with its owner) while being "accessed". If this is expected, acquire/release access 
// overloads must be used with a destruction sentinel stored on the callstack. If destroyed, `~FMRSWRecursiveAccessDetector()` will 
// clean up after the destroyed instance and notify the user (`FDestructionSentinel::bDestroyed`) that the release method can't be called as 
// the instance was destroyed.
public:
	enum class EAccessType { Reader, Writer };

	struct FDestructionSentinel
	{
		UE_FORCEINLINE_HINT explicit FDestructionSentinel(EAccessType InAccessType)
			: AccessType(InAccessType)
		{
		}

		EAccessType AccessType;
		const FMRSWRecursiveAccessDetector* Accessor;
		bool bDestroyed = false;
	};

private:
	// a TLS stack of destruction sentinels shared by all access detector instances on the callstack. the same access detector instance can have
	// multiple destruction sentinels on the stack. the next "release access" with destruction sentinel always matches the top of the stack
	using FDestructionSentinelStackTls = TArray<FDestructionSentinel*, TInlineAllocator<4>>;
	
	static CORE_API FDestructionSentinelStackTls& GetDestructionSentinelStackTls();
//////////////////////////////////////////////////////////////////////

private:
	///////////////////////////////////////////////
	// to support a write access from inside a read access on the same thread, we need to know that there're no readers on other threads.
	// As we can't have TLS slot per access detector instance, a single one is shared between multiple instances with the assumption that
	// there rarely will be more than one reader registered.
	struct FReaderNum
	{
		const FMRSWRecursiveAccessDetector* Reader;
		uint32 Num;
	};

	using FReadersTls = TArray<FReaderNum, TInlineAllocator<4>>;
	
	static CORE_API FReadersTls& GetReadersTls();

	void RemoveReaderFromTls() const
	{
		int32 ReaderIndex = GetReadersTls().IndexOfByPredicate([this](FReaderNum ReaderNum) { return ReaderNum.Reader == this; });

		ensureMsgf(ReaderIndex != INDEX_NONE,
			TEXT("Either invalid usage of the access detector (no matching AcquireReadAccess()) or the access detector was trivially relocated (this instance is not registered): %u readers, %u writers on thread %u:\nCurrent thread %u callstack:\n%s"),
			LoadState().ReaderNum, LoadState().WriterNum, LoadState().GetWriterThreadId(), 
			FPlatformTLS::GetCurrentThreadId(), *GetCurrentThreadCallstack());

		if (ReaderIndex == INDEX_NONE)
		{
			return; // to avoid asserting on the next line inside TArray
		}

		uint32 ReaderNum = --GetReadersTls()[ReaderIndex].Num;
		if (ReaderNum == 0)
		{
			GetReadersTls().RemoveAtSwap(ReaderIndex, EAllowShrinking::No);
		}
	}
	///////////////////////////////////////////////

public:
	FMRSWRecursiveAccessDetector() = default;

	UE_FORCEINLINE_HINT FMRSWRecursiveAccessDetector(const FMRSWRecursiveAccessDetector& Other)
		// just default initialization, the copy is not being accessed
	{
		CheckOtherThreadWriters(Other.LoadState());
	}

	inline FMRSWRecursiveAccessDetector& operator=(const FMRSWRecursiveAccessDetector& Other)
		// do not alter the state, it can be accessed
	{
		CheckOtherThreadWriters(Other.LoadState());
		return *this;
	}

	// use copy construction/assignment ^^ for move semantics too

	inline ~FMRSWRecursiveAccessDetector()
	{
		// search for all destruction sentinels for this instance and remove them from the stack, while building an expected correct state of
		// the access detector.
		FState ExpectedState;
		for (int DestructionSentinelIndex = 0; DestructionSentinelIndex != GetDestructionSentinelStackTls().Num(); ++DestructionSentinelIndex)
		{
			FDestructionSentinel& DestructionSentinel = *GetDestructionSentinelStackTls()[DestructionSentinelIndex];
			if (DestructionSentinel.Accessor == this)
			{
				DestructionSentinel.bDestroyed = true;
				if (DestructionSentinel.AccessType == EAccessType::Reader)
				{
					++ExpectedState.ReaderNum;

					RemoveReaderFromTls();
				}
				else
				{
					++ExpectedState.WriterNum;
					ExpectedState.SetWriterThreadId(FPlatformTLS::GetCurrentThreadId());
				}

				GetDestructionSentinelStackTls().RemoveAtSwap(DestructionSentinelIndex);
				--DestructionSentinelIndex;
			}
		}

		if (LoadState().Value != ExpectedState.Value)
		{
			ensureMsgf(UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure, 
				TEXT("Race detector destroyed while being accessed on another thread: %d readers, %d writers on thread %u:\nCurrent thread %u callstack:\n%s\nWriter thread %u callstack:\n%s"),
				LoadState().ReaderNum - ExpectedState.ReaderNum, 
				LoadState().WriterNum - ExpectedState.WriterNum, LoadState().GetWriterThreadId(), 
				FPlatformTLS::GetCurrentThreadId(), *GetCurrentThreadCallstack(),
				LoadState().GetWriterThreadId(), *GetThreadCallstack(LoadState().GetWriterThreadId()));
		}
	}

	inline bool AcquireReadAccess() const
	{
		FState PrevState = IncrementReaderNum();

		bool bSuccess = CheckOtherThreadWriters(PrevState);

		// register the reader in TLS
		int32 ReaderIndex = PrevState.ReaderNum == 0 ? 
			INDEX_NONE : 
			GetReadersTls().IndexOfByPredicate([this](FReaderNum ReaderNum) { return ReaderNum.Reader == this; });
		if (ReaderIndex == INDEX_NONE)
		{
			GetReadersTls().Add(FReaderNum{this, 1});
		}
		else
		{
			++GetReadersTls()[ReaderIndex].Num;
		}

		return bSuccess;
	}

	// an overload that handles access detector destruction from inside a read access, must be used along with the corresponding
	// overload of `ReleaseReadAccess`
	inline bool AcquireReadAccess(FDestructionSentinel& DestructionSentinel) const
	{
		DestructionSentinel.Accessor = this;
		GetDestructionSentinelStackTls().Add(&DestructionSentinel);

		return AcquireReadAccess();
	}

	inline bool ReleaseReadAccess() const
	{
		RemoveReaderFromTls();
		DecrementReaderNum();
		// no need to check for writers
		return true;
	}

	// an overload that handles access detector destruction from inside a read access, must be used along with the corresponding
	// overload of `AcquireReadAccess`
	inline bool ReleaseReadAccess(FDestructionSentinel& DestructionSentinel) const
	{
		bool bSuccess = ReleaseReadAccess();

		checkSlow(DestructionSentinel.Accessor == this);
		checkfSlow(GetDestructionSentinelStackTls().Num() != 0, TEXT("An attempt to remove a not registered destruction sentinel"));
		checkfSlow(GetDestructionSentinelStackTls().Last() == &DestructionSentinel, TEXT("Mismatched destruction sentinel: %p != %p"), GetDestructionSentinelStackTls().Last(), &DestructionSentinel);

		GetDestructionSentinelStackTls().RemoveAtSwap(GetDestructionSentinelStackTls().Num() - 1);

		return bSuccess;
	}

	inline bool AcquireWriteAccess()
	{
		FState LocalState = LoadState();
		bool bSuccess = true;

		if (LocalState.ReaderNum >= 1)
		{	// check that all readers are on the current thread
			int32 ReaderIndex = GetReadersTls().IndexOfByPredicate([this](FReaderNum ReaderNum) { return ReaderNum.Reader == this; });

			if (ReaderIndex == INDEX_NONE)
			{
				ensureMsgf(UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure, 
					TEXT("Either a race detected (%u reader(s) on another thread(s) while acquiring write access on the current thread) or the access detector was trivially relocated:\nCurrent thread %u callstack:\n%s"), 
					LocalState.ReaderNum, FPlatformTLS::GetCurrentThreadId(), *GetCurrentThreadCallstack());

				return false; // to avoid asserting on the next line inside TArray
			}

			if (GetReadersTls()[ReaderIndex].Num != LocalState.ReaderNum)
			{
				ensureMsgf(UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure,
					TEXT("Data race detected: %d reader(s) on another thread(s) while acquiring write access.\nCurrent thread %u callstack:\n%s"),
					LocalState.ReaderNum - GetReadersTls()[ReaderIndex].Num, FPlatformTLS::GetCurrentThreadId(), *GetCurrentThreadCallstack());
				
				bSuccess = false;
			}
		}

		uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
		if (LocalState.WriterNum != 0)
		{
			if (LocalState.GetWriterThreadId() != CurrentThreadId)
			{
				ensureMsgf(UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure,
					TEXT("Data race detected: acquiring write access on thread %u concurrently with %d writers on thread %u:\nCurrent thread %u callstack:\n%s\nWriter thread %u callstack:\n%s"), 
					CurrentThreadId, LocalState.WriterNum, LocalState.GetWriterThreadId(), 
					CurrentThreadId, *GetCurrentThreadCallstack(),
					LocalState.GetWriterThreadId(), *GetThreadCallstack(LocalState.GetWriterThreadId()));

				bSuccess = false;
			}
		}

		FState NewState{ LocalState.ReaderNum, LocalState.WriterNum + 1u, CurrentThreadId + 1 };
		FState PrevState = ExchangeState(NewState);

		if (LocalState != PrevState)
		{
			ensureMsgf(UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure,
				TEXT("Data race detected: other thread(s) activity during acquiring write access on thread %u: %u -> %u readers, %u -> %u writers on thread %u -> %u:\nCurrent thread %u callstack:\n%s\nWriter thread %u callstack:\n%s"),
				CurrentThreadId,
				LocalState.ReaderNum, PrevState.ReaderNum,
				LocalState.WriterNum, PrevState.WriterNum,
				LocalState.GetWriterThreadId(), PrevState.GetWriterThreadId(), 
				CurrentThreadId, *GetCurrentThreadCallstack(),
				PrevState.GetWriterThreadId(), *GetThreadCallstack(PrevState.GetWriterThreadId()));

			bSuccess = false;
		}

		return bSuccess;
	}

	// an overload that handles access detector destruction from inside a write access, must be used along with the corresponding
	// overload of `ReleaseWriteAccess`
	inline bool AcquireWriteAccess(FDestructionSentinel& DestructionSentinel)
	{
		DestructionSentinel.Accessor = this;
		GetDestructionSentinelStackTls().Add(&DestructionSentinel);

		return AcquireWriteAccess();
	}

	inline bool ReleaseWriteAccess()
	{
		FState LocalState = LoadState();

		uint32 WriterThreadId = LocalState.WriterNum != 1 ? LocalState.WriterThreadId : InvalidThreadId;
		FState NewState{ LocalState.ReaderNum, LocalState.WriterNum - 1u, WriterThreadId };
		FState PrevState = ExchangeState(NewState);

		if (LocalState != PrevState)
		{
			ensureMsgf(UE::Private::FMTAccessDetectorOptions::bSuppressCheckFailure,
				TEXT("Data race detected: other thread(s) activity during releasing write access on thread %d: %u -> %u readers, %u -> %u writers on thread %u -> %u\nCurrent thread %u callstack:\n%s\nWriter thread %u callstack:\n%s"),
				FPlatformTLS::GetCurrentThreadId(),
				LocalState.ReaderNum, PrevState.ReaderNum,
				LocalState.WriterNum, PrevState.WriterNum, 
				LocalState.GetWriterThreadId(), PrevState.GetWriterThreadId(),
				FPlatformTLS::GetCurrentThreadId(), *GetCurrentThreadCallstack(),
				PrevState.GetWriterThreadId(), *GetThreadCallstack(PrevState.GetWriterThreadId()));
			
			return false;
		}

		return true;
	}

	// an overload that handles access detector destruction from inside a write access, must be used along with the corresponding
	// overload of `AcquireWriteAccess`
	inline bool ReleaseWriteAccess(FDestructionSentinel& DestructionSentinel)
	{
		bool bSuccess = ReleaseWriteAccess();

		FDestructionSentinelStackTls& DestructionSentinelStackTls = GetDestructionSentinelStackTls();
		checkSlow(DestructionSentinel.Accessor == this);
		checkfSlow(GetDestructionSentinelStackTls().Num() != 0, TEXT("An attempt to remove a not registered destruction sentinel"));
		checkfSlow(GetDestructionSentinelStackTls().Last() == &DestructionSentinel, TEXT("Mismatched destruction sentinel: %p != %p"), GetDestructionSentinelStackTls().Last(), &DestructionSentinel);

		GetDestructionSentinelStackTls().RemoveAtSwap(GetDestructionSentinelStackTls().Num() - 1);

		return bSuccess;
	}
};

///////////////////////////////////////////////////////////////////

#define UE_MT_DECLARE_RW_ACCESS_DETECTOR(AccessDetector) FRWAccessDetector AccessDetector
#define UE_MT_DECLARE_RW_RECURSIVE_ACCESS_DETECTOR(AccessDetector) FRWRecursiveAccessDetector AccessDetector
#define UE_MT_DECLARE_RW_FULLY_RECURSIVE_ACCESS_DETECTOR(AccessDetector) FRWFullyRecursiveAccessDetector AccessDetector
#define UE_MT_DECLARE_MRSW_RECURSIVE_ACCESS_DETECTOR(AccessDetector) FMRSWRecursiveAccessDetector AccessDetector

#define UE_MT_SCOPED_READ_ACCESS(AccessDetector) const FBaseScopedAccessDetector& UE_JOIN(ScopedMTAccessDetector_,__LINE__) = MakeScopedReaderAccessDetector(AccessDetector)
#define UE_MT_SCOPED_WRITE_ACCESS(AccessDetector) const FBaseScopedAccessDetector& UE_JOIN(ScopedMTAccessDetector_,__LINE__) = MakeScopedWriterAccessDetector(AccessDetector)

#define UE_MT_ACQUIRE_READ_ACCESS(AccessDetector) (AccessDetector).AcquireReadAccess()
#define UE_MT_RELEASE_READ_ACCESS(AccessDetector) (AccessDetector).ReleaseReadAccess()
#define UE_MT_ACQUIRE_WRITE_ACCESS(AccessDetector) (AccessDetector).AcquireWriteAccess()
#define UE_MT_RELEASE_WRITE_ACCESS(AccessDetector) (AccessDetector).ReleaseWriteAccess()

#else // ENABLE_MT_DETECTOR

#define UE_MT_DECLARE_RW_ACCESS_DETECTOR(AccessDetector)
#define UE_MT_DECLARE_RW_RECURSIVE_ACCESS_DETECTOR(AccessDetector)
#define UE_MT_DECLARE_RW_FULLY_RECURSIVE_ACCESS_DETECTOR(AccessDetector)
#define UE_MT_DECLARE_MRSW_RECURSIVE_ACCESS_DETECTOR(AccessDetector)

#define UE_MT_SCOPED_READ_ACCESS(AccessDetector) 
#define UE_MT_SCOPED_WRITE_ACCESS(AccessDetector)

#define UE_MT_ACQUIRE_READ_ACCESS(AccessDetector)
#define UE_MT_RELEASE_READ_ACCESS(AccessDetector)
#define UE_MT_ACQUIRE_WRITE_ACCESS(AccessDetector)
#define UE_MT_RELEASE_WRITE_ACCESS(AccessDetector)

#endif // ENABLE_MT_DETECTOR
