// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM.h"
#include "AutoRTFM/OpenWrapper.h"
#include "HAL/Platform.h"
#include "MTAccessDetector.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"

#if ENABLE_MT_DETECTOR

// A transactionally safe access detector that works in the following novel ways:
// - In the open (non-transactional):
//   - Acquires the read/write access like before.
//   - Release the read/write access like before.
// - In the closed (transactional):
//   - During acquiring read access we query `ReadLockCount`:
//     - 0 means we haven't taken read access before in our transaction nest and we acquire it.
//     - But **only** if we haven't previously taken write access (by querying `WriteLockCount`).
//     - Then we bump `ReadLockCount` to remember we did a read.
//     - We also register an on-abort handler to release the access.
//   - During acquiring write access we query `WriteLockCount`:
//     - 0 means we haven't taken write access before in our transaction nest and we acquire it.
//     - But if `ReadLockCount` was non-zero then we have to upgrade the access from read to write.
//     - Then we bump `WriteLockCount` to remember we did a write.
//   - During releases we defer these to on-commit.
// 
// During on-commit we always release all our `ReadLockCount`'s first, so that we handle the case
// correctly where we had read then write, we need to only actually release the write access and
// this means we correctly handle that case.
struct FRWTransactionallySafeAccessDetectorDefinition
{
	FRWTransactionallySafeAccessDetectorDefinition() : State(MakeShared<FState>())
	{
	}

	// All the definitions in here are private because we do not want arbitrary code
	// to call the acquire/release read/write access functions directly, and instead
	// they should use `UE_MT_SCOPED_READ_ACCESS` and `UE_MT_SCOPED_WRITE_ACCESS`.
	// This restriction means that all acquires and releases are pairs that must
	// happen together in the open, or together in closed.
private:
	/**
	 * Acquires read access, will check if there are any writers
	 * @return true if no errors were detected
	 */
	inline bool AcquireReadAccess() const
	{
		if (!AutoRTFM::IsIdle())
		{
			const bool bResult = AutoRTFM::Open([&]
				{
					// The transactional system which can increment the lock counts
					// is always single-threaded, thus this is safe to check without atomicity.
					if (0 == State->ReadLockCount && 0 == State->WriteLockCount)
					{
						if (UNLIKELY(!State->Detector.AcquireReadAccess()))
						{
							return false;
						}
					}

					State->ReadLockCount += 1;
					return true;
				});

			if (UNLIKELY(!bResult))
			{
				return false;
			}

			RegisterOnAbortRelease();
			return true;
		}
		else
		{
			if (UNLIKELY(!State->Detector.AcquireReadAccess()))
			{
				return false;
			}

			ensure(0 == State->WriteLockCount);
			return true;
		}
	}

	/**
	 * Releases read access, will check if there are any writers
	 * @return true if no errors were detected
	 */
	inline bool ReleaseReadAccess() const
	{
		if (!AutoRTFM::IsIdle())
		{
			RegisterOnCommitRelease();

			// We can't do anything better here than returning true, because we
			// are deferring the actual release until on commit!
			return true;
		}
		else
		{
			ensure(0 == State->WriteLockCount);
			return State->Detector.ReleaseReadAccess();
		}
	}

	/** 
	 * Acquires write access, will check if there are readers or other writers
	 * @return true if no errors were detected
	 */
	inline bool AcquireWriteAccess() const
	{
		if (!AutoRTFM::IsIdle())
		{
			const bool bResult = AutoRTFM::Open([&]
				{
					if ((0 == State->ReadLockCount) && (0 == State->WriteLockCount))
					{
						// There have been no prior calls to `AcquireReadAccess`
						// so we can just claim write access directly.
						if (UNLIKELY(!State->Detector.AcquireWriteAccess()))
						{
							return false;
						}
					}
					else if (0 == State->WriteLockCount)
					{
						// There was a prior call to `AcquireReadAccess` so we need
						// to upgrade our read access to write access.
						if (UNLIKELY(!State->Detector.UpgradeReadAccessToWriteAccess()))
						{
							return false;
						}
					}

					State->WriteLockCount += 1;
					return true;
				});

			if (UNLIKELY(!bResult))
			{
				return false;
			}

			RegisterOnAbortRelease();
			return true;
		}
		else
		{
			if (UNLIKELY(!State->Detector.AcquireWriteAccess()))
			{
				return false;
			}

			ensure((0 == State->ReadLockCount) && (0 == State->WriteLockCount));
			return true;
		}
	}

	/** 
	 * Releases write access, will check if there are readers or other writers
	 * @return true if no errors were detected
	 */
	inline bool ReleaseWriteAccess() const
	{
		if (!AutoRTFM::IsIdle())
		{
			RegisterOnCommitRelease();

			// We can't do anything better here than returning true, because we
			// are deferring the actual release until on commit!
			return true;
		}
		else
		{
			ensure((0 == State->ReadLockCount) && (0 == State->WriteLockCount));
			return State->Detector.ReleaseWriteAccess();
		}
	}

private:
	struct FState final
	{
		// The underlying FRWAccessDetector.
		FRWAccessDetector Detector;

		// True if the abort handler has been registered.
		bool AbortHandlerRegistered = false;

		unsigned ReadLockCount = 0;

		unsigned WriteLockCount = 0;
		
		// Constructor is always open because FRWAccessDetector is not transactionally safe.
		UE_AUTORTFM_ALWAYS_OPEN
		FState() = default;
		
		// Destructor is always open because FRWAccessDetector is not transactionally safe.
		UE_AUTORTFM_ALWAYS_OPEN
		~FState()
		{
			ensure(0 == ReadLockCount);
			ensure(0 == WriteLockCount);
		}
	};

	UE_AUTORTFM_NOAUTORTFM static void ReleaseAccess(TSharedPtr<FState> State)
	{
		if (0 < State->ReadLockCount)
		{
			State->ReadLockCount -= 1;

			if ((0 == State->ReadLockCount) && (0 == State->WriteLockCount))
			{
				State->Detector.ReleaseReadAccess();
			}
		}
		else if (0 < State->WriteLockCount)
		{
			State->WriteLockCount -= 1;

			if (0 == State->WriteLockCount)
			{
				State->Detector.ReleaseWriteAccess();
			}
		}
		else
		{
			// We should only register as many on-abort handlers as we had lock count increments!
			check(false);
		}
	}

	UE_FORCEINLINE_HINT void RegisterOnAbortRelease() const
	{
		// We explicitly copy the state here for the case that `this` was stack
		// allocated and has already died before the on-abort is hit.
		AutoRTFM::OnAbort([State = AutoRTFM::TOpenWrapper{this->State}]
		{
			ReleaseAccess(State.Object);
		});
	}

	UE_FORCEINLINE_HINT void RegisterOnCommitRelease() const
	{
		// We explicitly copy the state here for the case that `this` was stack
		// allocated and has already died before the on-abort is hit.
		AutoRTFM::OnCommit([State = AutoRTFM::TOpenWrapper{this->State}]
		{
			ReleaseAccess(State.Object);
		});
	}

	// The state held for calls made when in a transaction.
	TSharedPtr<FState> State;

	friend TScopedReaderAccessDetector<FRWTransactionallySafeAccessDetectorDefinition>;
	friend TScopedWriterDetector<FRWTransactionallySafeAccessDetectorDefinition>;
	friend TScopedReaderAccessDetector<const FRWTransactionallySafeAccessDetectorDefinition>;
	friend TScopedWriterDetector<const FRWTransactionallySafeAccessDetectorDefinition>;
};

// TODO: if we made `FRWAccessDetector` have private + friend like below we don't need this anymore!
struct FRWFallbackSafeAccessDetectorDefinition : private FRWAccessDetector
{
	friend TScopedReaderAccessDetector<FRWFallbackSafeAccessDetectorDefinition>;
	friend TScopedWriterDetector<FRWFallbackSafeAccessDetectorDefinition>;
	friend TScopedReaderAccessDetector<const FRWFallbackSafeAccessDetectorDefinition>;
	friend TScopedWriterDetector<const FRWFallbackSafeAccessDetectorDefinition>;
};

#if UE_AUTORTFM
using FRWTransactionallySafeAccessDetector = FRWTransactionallySafeAccessDetectorDefinition;
#else
using FRWTransactionallySafeAccessDetector = FRWFallbackSafeAccessDetectorDefinition;
#endif

#define UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(AccessDetector) FRWTransactionallySafeAccessDetector AccessDetector;

#else // ENABLE_MT_DETECTOR

#define UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(AccessDetector)

#endif // ENABLE_MT_DETECTOR
