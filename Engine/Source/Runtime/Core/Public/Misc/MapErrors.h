// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/TokenizedMessage.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

/**
 * This file contains known map errors that can be referenced by name.
 * Documentation for these errors is assumed to lie in UDN at: Engine\Documentation\Source\Shared\Editor\MapErrors
 */
struct FMapErrors
{
    /**  Lighting */

	/** {LightActor} has same light GUID as {LightActor} (Duplicate and replace the orig with the new one): Duplicate and replace the original with the new one. */
    static CORE_API FLazyName MatchingLightGUID;
 
    /**  Actor */

	/** {Actor} has same GUID as {Actor} (Duplicate and replace the orig with the new one): Duplicate and replace the original with the new one. */
    static CORE_API FLazyName MatchingActorGUID;

	/** {Actor} : Large actor casts a shadow and will cause an extreme performance hit unless bUseBooleanEnvironmentShadowing is set to true: A large actor has been set to cast shadows - this will cause extreme performance issues and should have bUseBooleanEnvironmentShadowing set to true. */
    static CORE_API FLazyName ActorLargeShadowCaster;

	 /** {Volume} causes damage, but has no damagetype defined.: This warning is caused when there is a volume that is set to cause damage but doesn't have a damage type defined.  A damage type is important because it tells the game code how to handle a actor's reaction to taking damage.  This can be solved by going to the actor's Property Window->Physics Volume and setting the 'DamageType' property. */
    static CORE_API FLazyName NoDamageType;
   
	/** {Actor} : Brush has non-coplanar polygons: This warning is caused when there is a brush in the level that has non-coplanar polygons.  This is usually caused by using brush editing tools in geometry mode in extreme ways and can cause missing polygons in the level.  This warning can be resolved by deleting the brush and recreating it. */
    static CORE_API FLazyName NonCoPlanarPolys;
    
	/** {Actor} in same location as {Another Actor}: This warning is caused when there is a actor that is in the same exact location as another actor.  This warning is usually the result of a accidental duplication or paste operation.  It can be fixed by deleting one of the actors, or disregarded if the placement was intentional. */
    static CORE_API FLazyName SameLocation;
    
	/** {Actor} has invalid DrawScale/ DrawScale3D: This warning is caused when either DrawScale, DrawScale3D X, DrawScale3D Y, or DrawScale 3D Z is equal to zero.  Meaning that the actor will not be shown because it is being scaled to zero on one of its axis.  To solve this problem, change any DrawScale's that are zero to be non-zero by selecting the actor and changing its drawscale at the bottom of the main UnrealEd window. */
    static CORE_API FLazyName InvalidDrawscale;
    
	/** {Actor} is obsolete and must be removed!: This warning is caused when there is an instance of a actor in a level that has been marked deprecated.  This is usually because a actor was marked deprecated after the level was created, but the map was never updated.  This can be fixed by simply deleting the actor. */
    static CORE_API FLazyName ActorIsObselete;
    
	/** {Actor} bStatic true, but has Physics set to something other than PHYS_None!: This warning is caused when an actor has its bStatic flag set to true but its Physics is set to PHYS_None.  Since bStatic means that the actor will not be moving, having Physics set to PHYS_None is contradictory.  Actors set with the bStatic flag are also not ticked(updated).  This error can be solved by going to the actor's properties and changing its Physics to PHYS_None. */
    static CORE_API FLazyName StaticPhysNone;
    
	/** {Actor} : Volume actor has NULL collision component - please delete: The specified volume actor has a NULL collision component and should probably be deleted. */
    static CORE_API FLazyName VolumeActorCollisionComponentNULL;
    
	/** {Actor} : Volume actor has a collision component with 0 radius - please delete: The specified volume actor has a zero radius for its collision component and should probably be deleted. */
    static CORE_API FLazyName VolumeActorZeroRadius;
    
	/** {Actor} (LOD {Index}) has hand-painted vertex colors that no longer match the original StaticMesh {StaticMesh}: It looks like the original mesh was changed since this instance's vertex colors were painted down - this may need a refresh. */
    static CORE_API FLazyName VertexColorsNotMatchOriginalMesh;

	/** {ActorName} has collision enabled but StaticMesh ({StaticMeshName}) has no simple or complex collision. */
	static CORE_API FLazyName CollisionEnabledNoCollisionGeom;

	/** Actor casts dynamic shadows and has a BoundsScale greater than 1! This will have a large performance hit: Serious performance warning... either reduce BoundsScale to be <= 1 or remove dynamic shadows... */
    static CORE_API FLazyName ShadowCasterUsingBoundsScale;
    
	static CORE_API FLazyName MultipleSkyLights;

	static CORE_API FLazyName MultipleSkyAtmospheres;
	static CORE_API FLazyName MultipleSkyAtmosphereTypes;

	/** Actor gas invalid hair strands material */
	static CORE_API FLazyName InvalidHairStrandsMaterial;

	/** {ActorName} has WorldTrace blocked. It will be considered to be world geometry */
	static CORE_API FLazyName InvalidTrace;

    /**  BSP Brush */

	/** {Brush Actor} : Brush has zero polygons - please delete!: This warning indicates that you have a brush in the level that doesn't have any polygons associated with it.  The brush should be deleted as it isn't doing anything useful. */
    static CORE_API FLazyName BrushZeroPolygons;
    
	/** {Brush Actor} : Brush has NULL polygons property - please delete!: This warning indicates that you have a brush in the level with a "None" polygons reference.  The brush should be deleted as it isn't doing anything useful. */
	static CORE_API FLazyName BrushNullPolygons;
    
	/** Run 'Clean BSP Materials' to clear {count} references: This warning indicates that there are material references on brush faces that aren't contributing to the BSP, and that applying the Tools->'Clean BSP Materials' operation can clean up these references. */
    static CORE_API FLazyName CleanBSPMaterials;
    
	/** {Actor} : Brush has NULL BrushComponent property - please delete!: This warning is caused when there is a Brush with a "None" ConstraintInstance component.  It is usually found in older maps where duplication was used to create a Brush and can be fixed by deleting the Brush causing the warning and creating a new one. */
    static CORE_API FLazyName BrushComponentNull;
    
	/** {Brush} : Brush is planar: Planar brush used - please note that this may not work well with collision. */
    static CORE_API FLazyName PlanarBrush;

	/** Camera */

	/** Camera has AspectRatio=0 - please set this to something non-zero */
	static CORE_API FLazyName CameraAspectRatioIsZero;

    /**  Class */

	/** { }::{ } is obsolete and must be removed (Class is abstract): NEEDS DESCRIPTION */
    static CORE_API FLazyName AbstractClass;
    
	/** { }::{ } is obsolete and must be removed (Class is deprecated): NEEDS DESCRIPTION */
    static CORE_API FLazyName DeprecatedClass;
    

	/** Foliage */

	/** Foliage instances for a missing static mesh have been removed.: **TODO** */
	static CORE_API FLazyName FoliageMissingStaticMesh;

	/** Foliage in this map is missing {MissingCount} cluster component(s) for static mesh {MeshName}. Opening the Foliage tool will fix this problem.: **TODO** */
	static CORE_API FLazyName FoliageMissingClusterComponent;

    /**  Landscape */

	/** {LandscapeComponent} : Fixed up deleted layer weightmap: */
    static CORE_API FLazyName FixedUpDeletedLayerWeightmap;
    
	/** {LandscapeComponent} : Fixed up target layer info object not matching with the parent landscape's: */
	static CORE_API FLazyName FixedUpNonMatchingTargetLayer;

	/** {LandscapeComponent} : Fixed up incorrect layer weightmap texture index: */
    static CORE_API FLazyName FixedUpIncorrectLayerWeightmap;
    
	/** {LandscapeComponent} : Fixed up invalid material instances for a landscape component when its weightmap allocations don't match the material's layer configuration : */
	static CORE_API FLazyName FixedUpInvalidLandscapeMaterialInstances;

	/** Fixed up shared weightmap texture for layer {Layer} in component {Component} (shares with {Name}): */
    static CORE_API FLazyName FixedUpSharedLayerWeightmap;

	/** Landscape ({ProxyName}) has overlapping render components at location ({X, Y}): */
	static CORE_API FLazyName LandscapeComponentPostLoad_Warning;

	/** Landscape proxy {ProxyName} of {LevelName} points to a LandscapeActor that is not currently loaded. This will lose the property upon save.  */
	static CORE_API FLazyName LandscapeLazyObjectPtrDeprecation_Warning;

	/** Landscape proxy {Proxy} had one or more shared properties not in sync with its parent landscape actor {Parent}. This has been fixed but the proxy needs to be saved in order to ensure cooking behaves as expected. */
	static CORE_API FLazyName LandscapeProxyFixupSharedData_Warning;

	/** The landscape proxy property has been overridden but is invalid.  */
	static CORE_API FLazyName LandscapeProxyInvalidPropertyOverride_Warning;

	/** At least one of the landscape grass type overrides is invalid.  */
	static CORE_API FLazyName LandscapeProxyInvalidGrassTypesOverride_Warning;

    /**  Level */

	/** Duplicate level info: Two WorldInfos somehow exist... */
    static CORE_API FLazyName DuplicateLevelInfo;
    
	/** Map should have KillZ set.: This warning is caused when the map's KillZ in the WorldInfo properties is set to the default value. All maps should specify a KillZ appropriate for the level so that players cannot simply fall forever until they reach the playable world bounds. */
    static CORE_API FLazyName NoKillZ;
  

    /**  Lighting */

	/** {Actor} : Light actor has NULL LightComponent property - please delete!: This warning is caused when there is a Light actor with a "None" LightComponent.  It is usually found in older maps where duplication was used to create the actor and can be fixed by deleting the actor causing the warning and creating a new one. */
    static CORE_API FLazyName LightComponentNull;
    
	/** Maps need lighting rebuilt: This warning is caused when lighting has been invalidated by moving or modifying a light actor.  This can cause problems because the rendered lighting in the level is not accurately representing the current state of lights in the level.  This error can be solved by going to the Build menu and rebuilding lighting for a map. */
    static CORE_API FLazyName RebuildLighting;
   
	/** Component is a static type but has invalid lightmap settings!  Indirect lighting will be black.  Common causes are lightmap resolution of 0, LightmapCoordinateIndex out of bounds. */
	static CORE_API FLazyName StaticComponentHasInvalidLightmapSettings;
   
	/** Component is a using CastVolumetricTranslucentShadow but this feature is disabled for the project! Turn on r.Shadow.TranslucentPerObject.ProjectEnabled in a project ini if required. */
	static CORE_API FLazyName PrimitiveComponentHasInvalidTranslucentShadowSetting;	
	
	/** bUseFauxOrthoViewPos has been deprecated. Property will be lost on re-save. */
	static CORE_API FLazyName UseFauxOrthoViewPosDeprecation_Warning;

	/** {LightComponent} is using static lighting while Lumen is enabled. Static lights will fail to render. */
	static CORE_API FLazyName StaticLightWithLumen_Warning;

	/** Navigation */

	/** Paths need to be rebuilt: **TODO** */
	static CORE_API FLazyName RebuildPaths;

    /**  Particle System */

	/** {Actor} : Emitter actor has NULL ParticleSystemComponent property - please delete!: This warning is caused when there is a Emitter actor with a "None" ParticleSystemComponent.  It is usually found in older maps where duplication was used to create the actor and can be fixed by deleting the actor causing the warning and creating a new one. */
    static CORE_API FLazyName ParticleSystemComponentNull;
    
	/** PSysComp has an empty parameter actor reference at index {Index} ({Actor}): Param.Actor should not be NULL. */
    static CORE_API FLazyName PSysCompErrorEmptyActorRef;
    
	/** PSysComp has an empty parameter material reference at index {Index} ({Actor}): Param.Material should not be NULL. */
    static CORE_API FLazyName PSysCompErrorEmptyMaterialRef;
    

    /**  Skeletal Mesh */

	/** {Actor} : SkeletalMeshActor has no PhysicsAsset assigned.: In order for a SkeletalMesh to have an accurate bounding box, it needs to have a PhysicsAsset assigned in its SkeletalMeshComponent. An incorrect or inaccurate bounding box can lead to the mesh vanishing when its origin is not in view, or to poor shadow resolution because the bounding box is too big. */
    static CORE_API FLazyName SkelMeshActorNoPhysAsset;
    
	/** {Actor} : Skeletal mesh actor has NULL SkeletalMeshComponent property: The specified SkeletalMeshActor has a NULL SkeletalMeshComponent. */
    static CORE_API FLazyName SkeletalMeshComponent;
    
	/** {Actor} : Skeletal mesh actor has NULL SkeletalMesh property: The specified SkeletalMeshActor has a NULL SkeletalMesh. */
    static CORE_API FLazyName SkeletalMeshNull;

    /**  Sound */

	/** {Actor} : Ambient sound actor has NULL AudioComponent property - please delete!: This warning is caused when there is a Ambient sound actor with a "None" AudioComponent.  It is usually found in older maps where duplication was used to create a actor and can be fixed by deleting the actor causing the warning and creating a new one. */
    static CORE_API FLazyName AudioComponentNull;

	/** Ambient sound actor's AudioComponent has a NULL SoundCue property!: This warning is caused when there is a AmbientSound actor with a NULL SoundCue property.  This is a problem because the actor won't actually be playing any sounds.  This can be fixed by first choosing a sound cue in the generic browser and then going to the actor's Property Window->Audio Category->Audio Component and setting the 'SoundCue' property. */
    static CORE_API FLazyName SoundCueNull;

    /**  Static Mesh */

	/** {Static Mesh Actor} : Static mesh actor has NULL StaticMesh property: This warning is caused when there is a static mesh actor in a level with a NULL StaticMesh property.  This can be a problem because the actor exists and is using memory, but doesn't have a static mesh to actually draw.  This warning is usually the result of creating a StaticMesh actor without first selecting a StaticMesh in the generic browser.  This warning can be fixed by first selecting a static mesh in the generic browser and then going to the StaticMesh actor's Property Window->StaticMeshActor Category->StaticMeshComponent->StaticMeshComponent Category to set the 'StaticMesh' Property. */
    static CORE_API FLazyName StaticMeshNull;
    
	/** {Actor} : Static mesh actor has NULL StaticMeshComponent property - please delete!: This warning is caused when there is a static mesh actor with a "None" StaticMeshComponent component.  It is usually found in older maps where duplication was used to create the actor and can be fixed by deleting the actor causing the warning and creating a new one. */
    static CORE_API FLazyName StaticMeshComponent;
    
	/** {StaticMesh} has simple collision but is being scaled non-uniformly - collision creation will fail: Simple collision cannot be used with non-uniform scale. Please either fix the scale or the collision type. */
    static CORE_API FLazyName SimpleCollisionButNonUniformScale;

	/** More overridden materials {Count} on static mesh component than are referenced {Count} in source mesh {StaticMesh}: **TODO** */
    static CORE_API FLazyName MoreMaterialsThanReferenced;

	/** More Nanite overridden materials {Count} on static mesh component than are referenced {Count} in source mesh {StaticMesh}: **TODO** */
	static CORE_API FLazyName MoreNaniteMaterialsThanReferenced;

	 /** {Count} element(s) with zero triangles in static mesh {StaticMesh}: **TODO** */
    static CORE_API FLazyName ElementsWithZeroTriangles;
  
    /**  Volume */

	/** LevelStreamingVolume is not in the persistent level - please delete: This warning is caused when there is a level streaming volume that does not exist in the persistent level.  This can be problematic because the volume will not be considered when checking to see if a streaming level should be loaded or unloaded.  You can fix this problem by deleting the level streaming volume and recreating it. */
    static CORE_API FLazyName LevelStreamingVolume;
    
	/** No levels are associated with streaming volume.: This warning is caused when there are no levels associated with a LevelStreamingVolume, making it non-functional. This problem can be fixed by associating one or more streaming levels with the offending LevelStreamingVolume. */
    static CORE_API FLazyName NoLevelsAssociated;
    

    /**  Uncategorized */

	/** Filename is too long ({FilenameLength} characters) - this may interfere with cooking for consoles. Unreal filenames should be no longer than {Length} characters. Filename value: {Filename}: Please rename the file to be within the length specified. */
    static CORE_API FLazyName FilenameIsTooLongForCooking;
    
	/** {ObjectName} : Externally referenced */
	static CORE_API FLazyName UsingExternalObject;

    /**  Actors */

	/** {Actor} : Repaired painted vertex colors: Painted vertex colors were repaired on this actor. */
    static CORE_API FLazyName RepairedPaintedVertexColors;

	/** Hierarchical LOD **/

	/** {LODActor} : Static mesh is missing for the built LODActor.  Did you remove the asset? */
	static CORE_API FLazyName LODActorMissingStaticMesh;

	/** {LODActor} : Actor is missing. The actor might have been removed. We recommend you to build LOD again. */
	static CORE_API FLazyName LODActorMissingActor;

	/** {LODActor} : NoActor is assigned. We recommend to delete this actor. */
	static CORE_API FLazyName LODActorNoActorFound;

	/** Virtual Textures */

	/** {ObjectName} is using a virtual texture ('{TextureName}') on an unsupported property ('{PropertyName}'). */
	static CORE_API FLazyName InvalidVirtualTextureUsage;
};

/**
 * Map error specific message token.
 */
class FMapErrorToken : public FDocumentationToken
{
public:
	/** Factory method, tokens can only be constructed as shared refs */
	CORE_API static TSharedRef<FMapErrorToken> Create(const FName& InErrorName);

private:
	/** Private constructor */
	FMapErrorToken( const FName& InErrorName );
};
