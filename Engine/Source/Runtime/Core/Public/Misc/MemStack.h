// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/LockFreeFixedSizeAllocator.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "HAL/MemoryBase.h"
#include "HAL/PlatformCrt.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/ThreadSingleton.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Build.h"
#include "Misc/NoopCounter.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/UnrealTemplate.h"


// Enums for specifying memory allocation type.
enum EMemZeroed
{
	MEM_Zeroed = 1
};


enum EMemOned
{
	MEM_Oned = 1
};


class FPageAllocator
{
public:
	enum
	{
		PageSize = 64 * 1024,
		SmallPageSize = 1024-16 // allow a little extra space for allocator headers, etc
	};
#if UE_BUILD_SHIPPING
	typedef TLockFreeFixedSizeAllocator<PageSize, PLATFORM_CACHE_LINE_SIZE, FNoopCounter> TPageAllocator;
#else
	typedef TLockFreeFixedSizeAllocator<PageSize, PLATFORM_CACHE_LINE_SIZE, FThreadSafeCounter> TPageAllocator;
#endif

	static CORE_API FPageAllocator& Get();

	CORE_API ~FPageAllocator();
	CORE_API void* Alloc(int32 Alignment = MIN_ALIGNMENT);
	CORE_API void Free(void* Mem);
	CORE_API void* AllocSmall();
	CORE_API void FreeSmall(void* Mem);
	CORE_API uint64 BytesUsed();
	CORE_API uint64 BytesFree();
	CORE_API void LatchProtectedMode();
private:

	static FPageAllocator* Instance;
	static FPageAllocator& Construct();
	FPageAllocator();

#if STATS
	void UpdateStats();
#endif
	TPageAllocator TheAllocator;
};


/**
 * Simple linear-allocation memory stack.
 * Items are allocated via PushBytes() or the specialized operator new()s.
 * Items are freed en masse by using FMemMark to Pop() them.
 **/
class FMemStackBase
{
public:
	enum class EPageSize : uint8
	{
		// Small pages are allocated unless the allocation requires a larger page.
		Small,

		// Large pages are always allocated.
		Large
	};

	CORE_API FMemStackBase(EPageSize PageSize = EPageSize::Small);

	FMemStackBase(const FMemStackBase&) = delete;
	FMemStackBase(FMemStackBase&& Other)
	{
		*this = MoveTemp(Other);
	}

	FMemStackBase& operator=(FMemStackBase&& Other)
	{
		FreeChunks(nullptr);
		Top = Other.Top;
		End = Other.End;
		TopChunk = Other.TopChunk;
		TopMark = Other.TopMark;
		NumMarks = Other.NumMarks;
		bShouldEnforceAllocMarks = Other.bShouldEnforceAllocMarks;
		Other.Top = nullptr;
		Other.End = nullptr;
		Other.TopChunk = nullptr;
		Other.NumMarks = 0;
		Other.bShouldEnforceAllocMarks = false;
		return *this;
	}

	~FMemStackBase()
	{
		check((GIsCriticalError || !NumMarks));
		FreeChunks(nullptr);
	}

	UE_FORCEINLINE_HINT uint8* PushBytes(size_t AllocSize, size_t Alignment)
	{
		return (uint8*)Alloc(AllocSize, FMath::Max(AllocSize >= 16 ? (size_t)16 : (size_t)8, Alignment));
	}

	inline bool CanFitInPage(size_t AllocSize, size_t Alignment) const
	{
		const uint8* Result = Align(Top, Alignment);
		const uint8* NewTop = Result + AllocSize;
		return NewTop <= End;
	}

	inline void* Alloc(size_t AllocSize, size_t Alignment)
	{
		// Debug checks.
		checkSlow(AllocSize>=0);
		checkSlow((Alignment&(Alignment-1))==0);
		checkSlow(Top<=End);
		check(!bShouldEnforceAllocMarks || NumMarks > 0);

		// Try to get memory from the current chunk.
		uint8* Result = Align( Top, Alignment );
		uint8* NewTop = Result + AllocSize;

		// Make sure we didn't overflow.
		if ( NewTop <= End )
		{
			Top	= NewTop;
		}
		else
		{
			// We'd pass the end of the current chunk, so allocate a new one.
			AllocateNewChunk( AllocSize + Alignment );
			Result = Align( Top, Alignment );
			NewTop = Result + AllocSize;
			Top = NewTop;
		}
		return Result;
	}

	UE_FORCEINLINE_HINT uint8* GetTop() const
	{
		return Top;
	}

	/** return true if this stack is empty. */
	UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return TopChunk == nullptr;
	}

	inline void Flush()
	{
		check(!NumMarks);
		FreeChunks(nullptr);
	}
	UE_FORCEINLINE_HINT int32 GetNumMarks()
	{
		return NumMarks;
	}
	/** @return the number of bytes allocated for this FMemStack that are currently in use. */
	CORE_API int32 GetByteCount() const;

	// Returns true if the pointer was allocated using this allocator
	CORE_API bool ContainsPointer(const void* Pointer) const;

	// Friends.
	friend class FMemMark;
	friend void* operator new(size_t Size, FMemStackBase& Mem, int32 Count);
	friend void* operator new(size_t Size, std::align_val_t Align, FMemStackBase& Mem, int32 Count);
	friend void* operator new(size_t Size, FMemStackBase& Mem, EMemZeroed Tag, int32 Count);
	friend void* operator new(size_t Size, std::align_val_t Align, FMemStackBase& Mem, EMemZeroed Tag, int32 Count);
	friend void* operator new(size_t Size, FMemStackBase& Mem, EMemOned Tag, int32 Count);
	friend void* operator new(size_t Size, std::align_val_t Align, FMemStackBase& Mem, EMemOned Tag, int32 Count);
	friend void* operator new[](size_t Size, FMemStackBase& Mem, int32 Count);
	friend void* operator new[](size_t Size, std::align_val_t Align, FMemStackBase& Mem, int32 Count);
	friend void* operator new[](size_t Size, FMemStackBase& Mem, EMemZeroed Tag, int32 Count);
	friend void* operator new[](size_t Size, std::align_val_t Align, FMemStackBase& Mem, EMemZeroed Tag, int32 Count);
	friend void* operator new[](size_t Size, FMemStackBase& Mem, EMemOned Tag, int32 Count);
	friend void* operator new[](size_t Size, std::align_val_t Align, FMemStackBase& Mem, EMemOned Tag, int32 Count);

	// Types.
	struct FTaggedMemory
	{
		FTaggedMemory* Next;
		int32 DataSize;

		uint8 *Data() const
		{
			return ((uint8*)this) + sizeof(FTaggedMemory);
		}
	};

private:

	/**
	 * Allocate a new chunk of memory of at least MinSize size,
	 * updates the memory stack's Chunks table and ActiveChunks counter.
	 */
	CORE_API void AllocateNewChunk( int32 MinSize );

	/** Frees the chunks above the specified chunk on the stack. */
	CORE_API void FreeChunks( FTaggedMemory* NewTopChunk );

	// Variables.
	uint8*			Top = nullptr;				// Top of current chunk (Top<=End).
	uint8*			End = nullptr;				// End of current chunk.
	FTaggedMemory*	TopChunk = nullptr;			// Only chunks 0..ActiveChunks-1 are valid.

	/** The top mark on the stack. */
	class FMemMark*	TopMark = nullptr;

	/** The number of marks on this stack. */
	int32 NumMarks = 0;

	/** The page size to use when allocating. */
	EPageSize PageSize = EPageSize::Small;

protected:
	bool bShouldEnforceAllocMarks = false;
};

UE_DECLARE_THREAD_SINGLETON_TLS(class FMemStack, CORE_API)

class FMemStack : public TThreadSingleton<FMemStack>, public FMemStackBase
{
public:
	FMemStack()
	{
		bShouldEnforceAllocMarks = true;
	}
};


/*-----------------------------------------------------------------------------
	FMemStack templates.
-----------------------------------------------------------------------------*/

// Operator new for typesafe memory stack allocation.
template <class T> inline T* New(FMemStackBase& Mem, int32 Count = 1, int32 Align = DEFAULT_ALIGNMENT)
{
	return (T*)Mem.PushBytes( Count*sizeof(T), Align );
}
template <class T> inline T* NewZeroed(FMemStackBase& Mem, int32 Count = 1, int32 Align = DEFAULT_ALIGNMENT)
{
	uint8* Result = Mem.PushBytes( Count*sizeof(T), Align );
	FMemory::Memzero( Result, Count*sizeof(T) );
	return (T*)Result;
}
template <class T> inline T* NewOned(FMemStackBase& Mem, int32 Count = 1, int32 Align = DEFAULT_ALIGNMENT)
{
	uint8* Result = Mem.PushBytes( Count*sizeof(T), Align );
	FMemory::Memset( Result, 0xff, Count*sizeof(T) );
	return (T*)Result;
}


/*-----------------------------------------------------------------------------
	FMemStack operator new's.
-----------------------------------------------------------------------------*/

// Operator new for typesafe memory stack allocation.
inline void* operator new(size_t Size, FMemStackBase& Mem, int32 Count = 1)
{
	// Get uninitialized memory.
	const size_t SizeInBytes = Size * Count;
	checkSlow(SizeInBytes <= (size_t)TNumericLimits<int32>::Max());
	return Mem.PushBytes( SizeInBytes, __STDCPP_DEFAULT_NEW_ALIGNMENT__);
}
inline void* operator new(size_t Size, std::align_val_t Align, FMemStackBase& Mem, int32 Count = 1) // c++17
{
	// Get uninitialized memory.
	const size_t SizeInBytes = Size * Count;
	checkSlow(SizeInBytes <= (size_t)TNumericLimits<int32>::Max());
	return Mem.PushBytes(SizeInBytes, (size_t)Align);
}
inline void* operator new(size_t Size, FMemStackBase& Mem, EMemZeroed Tag, int32 Count = 1)
{
	// Get zero-filled memory.
	const size_t SizeInBytes = Size * Count;
	checkSlow(SizeInBytes <= (size_t)TNumericLimits<int32>::Max());
	uint8* Result = Mem.PushBytes( SizeInBytes, __STDCPP_DEFAULT_NEW_ALIGNMENT__);
	FMemory::Memzero( Result, SizeInBytes );
	return Result;
}
inline void* operator new(size_t Size, std::align_val_t Align, FMemStackBase& Mem, EMemZeroed Tag, int32 Count = 1) // c++17
{
	// Get zero-filled memory.
	const size_t SizeInBytes = Size * Count;
	checkSlow(SizeInBytes <= (size_t)TNumericLimits<int32>::Max());
	uint8* Result = Mem.PushBytes(SizeInBytes, (size_t)Align);
	FMemory::Memzero(Result, SizeInBytes);
	return Result;
}
inline void* operator new(size_t Size, FMemStackBase& Mem, EMemOned Tag, int32 Count = 1)
{
	// Get one-filled memory.
	const size_t SizeInBytes = Size * Count;
	checkSlow(SizeInBytes <= (size_t)TNumericLimits<int32>::Max());
	uint8* Result = Mem.PushBytes( SizeInBytes, __STDCPP_DEFAULT_NEW_ALIGNMENT__);
	FMemory::Memset( Result, 0xff, SizeInBytes );
	return Result;
}
inline void* operator new(size_t Size, std::align_val_t Align, FMemStackBase& Mem, EMemOned Tag, int32 Count = 1) // c++17
{
	// Get one-filled memory.
	const size_t SizeInBytes = Size * Count;
	checkSlow(SizeInBytes <= (size_t)TNumericLimits<int32>::Max());
	uint8* Result = Mem.PushBytes(SizeInBytes, (size_t)Align);
	FMemory::Memset(Result, 0xff, SizeInBytes);
	return Result;
}
inline void* operator new[](size_t Size, FMemStackBase& Mem, int32 Count = 1)
{
	// Get uninitialized memory.
	const size_t SizeInBytes = Size * Count;
	checkSlow(SizeInBytes <= (size_t)TNumericLimits<int32>::Max());
	return Mem.PushBytes( SizeInBytes, __STDCPP_DEFAULT_NEW_ALIGNMENT__);
}
inline void* operator new[](size_t Size, std::align_val_t Align, FMemStackBase& Mem, int32 Count = 1) // c++17
{
	// Get uninitialized memory.
	const size_t SizeInBytes = Size * Count;
	checkSlow(SizeInBytes <= (size_t)TNumericLimits<int32>::Max());
	return Mem.PushBytes(SizeInBytes, (size_t)Align);
}
inline void* operator new[](size_t Size, FMemStackBase& Mem, EMemZeroed Tag, int32 Count = 1)
{
	// Get zero-filled memory.
	const size_t SizeInBytes = Size * Count;
	checkSlow(SizeInBytes <= (size_t)TNumericLimits<int32>::Max());
	uint8* Result = Mem.PushBytes(SizeInBytes, __STDCPP_DEFAULT_NEW_ALIGNMENT__);
	FMemory::Memzero( Result, SizeInBytes );
	return Result;
}
inline void* operator new[](size_t Size, std::align_val_t Align, FMemStackBase& Mem, EMemZeroed Tag, int32 Count = 1) // c++17
{
	// Get zero-filled memory.
	const size_t SizeInBytes = Size * Count;
	checkSlow(SizeInBytes <= (size_t)TNumericLimits<int32>::Max());
	uint8* Result = Mem.PushBytes(SizeInBytes, (size_t)Align);
	FMemory::Memzero(Result, SizeInBytes);
	return Result;
}
inline void* operator new[](size_t Size, FMemStackBase& Mem, EMemOned Tag, int32 Count = 1)
{
	// Get one-filled memory.
	const size_t SizeInBytes = Size * Count;
	checkSlow(SizeInBytes <= (size_t)TNumericLimits<int32>::Max());
	uint8* Result = Mem.PushBytes( SizeInBytes, __STDCPP_DEFAULT_NEW_ALIGNMENT__);
	FMemory::Memset( Result, 0xff, SizeInBytes );
	return Result;
}
inline void* operator new[](size_t Size, std::align_val_t Align, FMemStackBase& Mem, EMemOned Tag, int32 Count = 1) // c++17
{
	// Get one-filled memory.
	const size_t SizeInBytes = Size * Count;
	checkSlow(SizeInBytes <= (size_t)TNumericLimits<int32>::Max());
	uint8* Result = Mem.PushBytes(SizeInBytes, (size_t)Align);
	FMemory::Memset(Result, 0xff, SizeInBytes);
	return Result;
}

namespace UE::Core::Private
{
	[[noreturn]] CORE_API void OnInvalidMemStackAllocatorNum(int32 NewNum, SIZE_T NumBytesPerElement);
}

/** A container allocator that allocates from a mem-stack. */
template<uint32 Alignment = DEFAULT_ALIGNMENT>
class TMemStackAllocator
{
public:
	using SizeType = int32;

	enum { NeedsElementType = true };
	enum { RequireRangeCheck = true };

	template<typename ElementType>
	class ForElementType
	{
	public:

		/** Default constructor. */
		ForElementType():
			Data(nullptr)
		{}

		/**
		 * Moves the state of another allocator into this one.
		 * Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		 * @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		 */
		inline void MoveToEmpty(ForElementType& Other)
		{
			checkSlow(this != &Other);

			Data       = Other.Data;
			Other.Data = nullptr;
		}

		// FContainerAllocatorInterface
		UE_FORCEINLINE_HINT ElementType* GetAllocation() const
		{
			return Data;
		}

		//@TODO: FLOATPRECISION: Takes SIZE_T input but doesn't actually support it
		void ResizeAllocation(SizeType CurrentNum, SizeType NewMax,SIZE_T NumBytesPerElement)
		{
			void* OldData = Data;
			if( NewMax )
			{
				static_assert(sizeof(int32) <= sizeof(SIZE_T), "SIZE_T is expected to be larger than int32");

				// Check for under/overflow
				if (UNLIKELY(NewMax < 0 || NumBytesPerElement < 1 || NumBytesPerElement > (SIZE_T)MAX_int32))
				{
					UE::Core::Private::OnInvalidMemStackAllocatorNum(NewMax, NumBytesPerElement);
				}

				// Allocate memory from the stack.
				Data = (ElementType*)FMemStack::Get().PushBytes(
					(int32)(NewMax * NumBytesPerElement),
					FMath::Max(Alignment,(uint32)alignof(ElementType))
					);

				// If the container currently holds elements, copy them into the new allocation.
				if(OldData && CurrentNum)
				{
					const SizeType NumCopiedElements = FMath::Min(NewMax,CurrentNum);
					FMemory::Memcpy(Data,OldData,NumCopiedElements * NumBytesPerElement);
				}
			}
		}
		UE_FORCEINLINE_HINT SizeType CalculateSlackReserve(SizeType NewMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackReserve(NewMax, NumBytesPerElement, false, Alignment);
		}
		UE_FORCEINLINE_HINT SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, false, Alignment);
		}
		UE_FORCEINLINE_HINT SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackGrow(NewMax, CurrentMax, NumBytesPerElement, false, Alignment);
		}

		UE_FORCEINLINE_HINT SIZE_T GetAllocatedSize(SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return CurrentMax * NumBytesPerElement;
		}

		bool HasAllocation() const
		{
			return !!Data;
		}

		SizeType GetInitialCapacity() const
		{
			return 0;
		}
			
	private:

		/** A pointer to the container's elements. */
		ElementType* Data;
	};
	
	typedef ForElementType<FScriptContainerElement> ForAnyElementType;
};

template <uint32 Alignment>
struct TAllocatorTraits<TMemStackAllocator<Alignment>> : TAllocatorTraitsBase<TMemStackAllocator<Alignment>>
{
	enum { IsZeroConstruct = true };
};


/**
 * FMemMark marks a top-of-stack position in the memory stack.
 * When the marker is constructed or initialized with a particular memory 
 * stack, it saves the stack's current position. When marker is popped, it
 * pops all items that were added to the stack subsequent to initialization.
 */
class FMemMark
{
public:
	// Constructors.
	FMemMark(FMemStackBase& InMem)
	:	Mem(InMem)
	,	Top(InMem.Top)
	,	SavedChunk(InMem.TopChunk)
	,	bPopped(false)
	,	NextTopmostMark(InMem.TopMark)
	{
		Mem.TopMark = this;

		// Track the number of outstanding marks on the stack.
		Mem.NumMarks++;
	}

	/** Destructor. */
	~FMemMark()
	{
		Pop();
	}

	/** Free the memory allocated after the mark was created. */
	void Pop()
	{
		if(!bPopped)
		{
			check(Mem.TopMark == this);
			bPopped = true;

			// Track the number of outstanding marks on the stack.
			--Mem.NumMarks;

			// Unlock any new chunks that were allocated.
			if( SavedChunk != Mem.TopChunk )
			{
				Mem.FreeChunks( SavedChunk );
			}

			// Restore the memory stack's state.
			Mem.Top = Top;
			Mem.TopMark = NextTopmostMark;

			// Ensure that the mark is only popped once by clearing the top pointer.
			Top = nullptr;
		}
	}

private:

	// Implementation variables.
	FMemStackBase& Mem;
	uint8* Top;
	FMemStackBase::FTaggedMemory* SavedChunk;
	bool bPopped;
	FMemMark* NextTopmostMark;
};
