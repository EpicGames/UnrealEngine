// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/MemStack.h"
#include "Misc/StringBuilder.h"
#include "Containers/ArrayView.h"
#include "Containers/StringView.h"

namespace UE
{
namespace MemStack
{

inline const TCHAR* AllocateString(FMemStackBase& Allocator, const TCHAR* String, int32 Length)
{
	TCHAR* Result = New<TCHAR>(Allocator, Length + 1);
	FMemory::Memcpy(Result, String, Length * sizeof(TCHAR));
	Result[Length] = 0;
	return Result;
}

inline const TCHAR* AllocateString(FMemStackBase& Allocator, FStringView String)
{
	return AllocateString(Allocator, String.GetData(), String.Len());
}

inline FStringView AllocateStringView(FMemStackBase& Allocator, FStringView String)
{
	return FStringView(AllocateString(Allocator, String), String.Len());
}

inline const TCHAR* AllocateString(FMemStackBase& Allocator, const FStringBuilderBase& StringBuilder)
{
	return AllocateString(Allocator, StringBuilder.GetData(), StringBuilder.Len());
}

template <typename FormatType, typename... ArgTypes>
inline const TCHAR* AllocateStringf(FMemStackBase& Allocator, const FormatType& Format, ArgTypes... Args)
{
	TStringBuilder<1024> String;
	String.Appendf(Format, Forward<ArgTypes>(Args)...);
	return AllocateString(Allocator, String);
}

template <typename FormatType, typename... ArgTypes>
inline FStringView AllocateStringViewf(FMemStackBase& Allocator, const FormatType& Format, ArgTypes... Args)
{
	TStringBuilder<1024> String;
	String.Appendf(Format, Forward<ArgTypes>(Args)...);
	return AllocateStringView(Allocator, String.ToView());
}

template<typename T>
inline TArrayView<T> AllocateArrayView(FMemStackBase& Allocator, TArrayView<T> View)
{
	T* Data = nullptr;
	if (View.Num() > 0)
	{
		Data = new(Allocator) T[View.Num()];
		for (int32 i = 0; i < View.Num(); ++i)
		{
			Data[i] = View[i];
		}
	}
	return MakeArrayView(Data, View.Num());
}

} // namespace MemStack
} // namespace UE
