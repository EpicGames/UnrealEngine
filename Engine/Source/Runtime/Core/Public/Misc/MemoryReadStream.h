// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/ThreadSafeCounter.h"
#include "Templates/RefCounting.h"

/**
 * Represents a block of memory, but abstracts away the underlying layout
 */
class IMemoryReadStream
{
public:
	static CORE_API TRefCountPtr<IMemoryReadStream> CreateFromCopy(const void* InMemory, int64 InSize);
	static CORE_API TRefCountPtr<IMemoryReadStream> CreateFromCopy(IMemoryReadStream* InStream);
	static CORE_API TRefCountPtr<IMemoryReadStream> CreateFromBuffer(void* InMemory, int64 InSize, bool bOwnPointer);

	virtual const void* Read(int64& OutSize, int64 InOffset, int64 InSize) = 0;
	virtual int64 GetSize() = 0;
	CORE_API virtual void CopyTo(void* Buffer, int64 InOffset, int64 InSize);
	virtual void EnsureReadNonBlocking() {};		// Incur any potential wait such that future Read() or CopyTo() calls are wait and thread switch free.

	UE_FORCEINLINE_HINT uint32 AddRef() const { return uint32(NumRefs.Increment()); }

	inline uint32 Release() const
	{
		const int32 Refs = NumRefs.Decrement();
		if (Refs == 0)
		{
			delete this;
		}
		return uint32(Refs);
	}

	UE_FORCEINLINE_HINT uint32 GetRefCount() const
	{
		return uint32(NumRefs.GetValue());
	}

protected:
	virtual ~IMemoryReadStream() = default;

private:
	mutable FThreadSafeCounter NumRefs;
};
typedef TRefCountPtr<IMemoryReadStream> IMemoryReadStreamRef;
