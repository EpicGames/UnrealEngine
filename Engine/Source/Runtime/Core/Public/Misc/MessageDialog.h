// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "HAL/PlatformMisc.h"

class FText;

/** 
 * FMessageDialog
 * These functions open a message dialog and display the specified informations
 * there.
 **/
struct FMessageDialog
{
	/** Pops up a message dialog box containing the input string.
	 * @param Message Text of message to show
	 * @param Title Optional title to use (defaults to "Message")
	*/
	static CORE_API void Debugf( const FText& Message );
	static CORE_API void Debugf( const FText& Message, const FText& Title );

	/** Pops up a message dialog box containing the last system error code in string form. */
	static CORE_API void ShowLastError();

	/**
	 * Open a modal message box dialog
	 * @param MessageCategory Controls the icon used for the dialog
	 * @param MessageType Controls buttons dialog should have
	 * @param Message Text of message to show
	 * @param Title Optional title to use (defaults to "Message")
	*/
	static CORE_API EAppReturnType::Type Open( EAppMsgType::Type MessageType, const FText& Message);
	static CORE_API EAppReturnType::Type Open( EAppMsgType::Type MessageType, const FText& Message, const FText& Title);
	static CORE_API EAppReturnType::Type Open( EAppMsgCategory MessageCategory, EAppMsgType::Type MessageType, const FText& Message);
	static CORE_API EAppReturnType::Type Open( EAppMsgCategory MessageCategory, EAppMsgType::Type MessageType, const FText& Message, const FText& Title);

	/**
	 * Open a modal message box dialog
	 * @param MessageCategory Controls the icon used for the dialog
	 * @param MessageType Controls buttons dialog should have
	 * @param DefaultValue If the application is Unattended, the function will log and return DefaultValue
	 * @param Message Text of message to show
	 * @param Title Optional title to use (defaults to "Message")
	*/
	static CORE_API EAppReturnType::Type Open(EAppMsgType::Type MessageType, EAppReturnType::Type DefaultValue, const FText& Message);
	static CORE_API EAppReturnType::Type Open(EAppMsgType::Type MessageType, EAppReturnType::Type DefaultValue, const FText& Message, const FText& Title);
	static CORE_API EAppReturnType::Type Open(EAppMsgCategory MessageCategory, EAppMsgType::Type MessageType, EAppReturnType::Type DefaultValue, const FText& Message);
	static CORE_API EAppReturnType::Type Open(EAppMsgCategory MessageCategory, EAppMsgType::Type MessageType, EAppReturnType::Type DefaultValue, const FText& Message, const FText& Title);
};
