// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "Misc/DateTime.h"
#include "Misc/SingleThreadRunnable.h"
#include "Misc/Timespan.h"

class FRunnableThread;

/**
 * Declares a delegate that is executed when a monitored process completed.
 *
 * The first parameter is the process return code.
 */
DECLARE_DELEGATE_OneParam(FOnMonitoredProcessCompleted, int32)

/**
 * Declares a delegate that is executed when a monitored process produces output.
 *
 * The first parameter is the produced output.
 */
DECLARE_DELEGATE_OneParam(FOnMonitoredProcessOutput, FString)


/**
 * Implements an external process that can be monitored.
 */
class FMonitoredProcess
	: public FRunnable, FSingleThreadRunnable
{
public:

	/**
	 * Creates a new monitored process.
	 *
	 * @param InURL The URL of the executable to launch.
	 * @param InParams The command line parameters.
	 * @param InHidden Whether the window of the process should be hidden.
	 * @param InCreatePipes Whether the output should be redirected to the caller.
	 */
	CORE_API FMonitoredProcess( const FString& InURL, const FString& InParams, bool InHidden, bool InCreatePipes = true );

	/**
	* Creates a new monitored process.
	*
	* @param InURL The URL of the executable to launch.
	* @param InParams The command line parameters.
	* @param InHidden Whether the window of the process should be hidden.
	* @param InWorkingDir The URL of the working dir where the executable should launch.
	* @param InCreatePipes Whether the output should be redirected to the caller.
	*/
	CORE_API FMonitoredProcess( const FString& InURL, const FString& InParams, const FString& InWorkingDir, bool InHidden, bool InCreatePipes = true );

	/**
	 * Helper method for synchronously executing a process and gathering the output
	 *
	 * @param InURL The URL of the executable to launch
	 * @param InParams The command line parameters
	 * @param OutOutput An array of strings containing the output
	 * @param Timeout An optional maximum time to wait for the process to exit before cancelling it.
	 * @return Returns true if the process completed successfully (return code == 0)
	 */
	static CORE_API bool RunProcessSynchronous(const FString& InURL, const FString& InParams, TArray<FString>& OutOutput, float Timeout = 0.f);

	/** Destructor. */
	CORE_API virtual ~FMonitoredProcess();

public:

	/**
	 * Cancels the process.
	 *
	 * @param InKillTree Whether to kill the entire process tree when canceling this process.
	 */
	void Cancel( bool InKillTree = false )
	{
		KillTree = InKillTree;
		Canceling = true;
	}

	/**
	 * Gets the duration of time that the task has been running.
	 *
	 * @return Time duration.
	 */
	CORE_API FTimespan GetDuration() const;

	/**
	 * Gets the Process Handle. The instance can be invalid if the process was not created.
	 *
	 * @return The Process Handle
	 */
	FProcHandle GetProcessHandle() const
	{
		return ProcessHandle;
	}

	/**
	 * Returns the commandline of the process which will be executed if Launch is called
	 */
	FString GetCommandline() const
	{
		return FString::Printf(TEXT("%s %s"), *URL, *Params);
	}

	/**
	* Checks whether the process is still running. In single threaded mode, this will tick the thread processing
	*
	* @return true if the process is running, false otherwise.
	*/
	CORE_API bool Update();

	/** Launches the process. */
	CORE_API virtual bool Launch();

	/**
	 * Sets the sleep interval to be used in the main thread loop.
	 *
	 * @param InSleepInterval The Sleep interval to use.
	 */
	void SetSleepInterval( float InSleepInterval )
	{
		SleepInterval = InSleepInterval;
	}

public:

	/**
	 * Returns a delegate that is executed when the process has been canceled.
	 *
	 * @return The delegate.
	 */
	FSimpleDelegate& OnCanceled()
	{
		return CanceledDelegate;
	}

	/**
	 * Returns a delegate that is executed when a monitored process completed.
	 *
	 * @return The delegate.
	 */
	FOnMonitoredProcessCompleted& OnCompleted()
	{
		return CompletedDelegate;
	}

	/**
	 * Returns a delegate that is executed when a monitored process produces output.
	 *
	 * @return The delegate.
	 */
	FOnMonitoredProcessOutput& OnOutput()
	{
		return OutputDelegate;
	}

	/**
	 * Returns the return code from the exited process
	 *
	 * @return Process return code
	 */
	CORE_API int GetReturnCode() const;

	/**
	 * Returns the full output, wihtout needing to hookup a delegate and buffer it externally. Note that if OutputDelegate is 
	 * bound, this _will not_ have the entire output
	 */
	CORE_API const FString& GetFullOutputWithoutDelegate() const;

public:

	// FRunnable interface

	virtual bool Init() override
	{
		return true;
	}

	CORE_API virtual uint32 Run() override;

	virtual void Stop() override
	{
		Cancel();
	}

	virtual void Exit() override { }

	virtual FSingleThreadRunnable* GetSingleThreadInterface() override
	{
		return this;
	}

protected:

	/**
	* FSingleThreadRunnable interface
	*/
	CORE_API void Tick() override;

	/**
	 * Processes the given output string.
	 *
	 * @param Output The output string to process.
	 */
	CORE_API void ProcessOutput( const FString& Output );

protected:
	CORE_API void TickInternal();


	// Whether the process is being canceled. */
	TAtomic<bool> Canceling = false;

	// Holds the time at which the process ended. */
	TAtomic<FDateTime> EndTime;

	// Whether the window of the process should be hidden. */
	bool Hidden = false;

	// Whether to kill the entire process tree when cancelling this process. */
	TAtomic<bool> KillTree = false;

	// Holds the command line parameters. */
	FString Params;

	// Holds the handle to the process. */
	FProcHandle ProcessHandle;

	// Holds the non-inheritable read end of stdout pipe. */
	void* ReadPipe = nullptr;

	// Holds the return code. */
	TAtomic<int> ReturnCode = 0;

	// Holds the time at which the process started. */
	TAtomic<FDateTime> StartTime { 0 };

	// Holds the monitoring thread object. */
	FRunnableThread* Thread = nullptr;

	// Is the thread running? 
	TAtomic<bool> bIsRunning;

	// Holds the URL of the executable to launch. */
	FString URL;

	// Holds the URL of the working dir for the process. */
	FString WorkingDir;

	// Holds the inheritable write end of stdout pipe. */
	void* WritePipe = nullptr;

	// Holds if we should create pipes
	bool bCreatePipes = false;

	// Sleep interval to use
	float SleepInterval = 0.01f;

	// Buffered output text which does not contain a newline
	FString OutputBuffer;

protected:

	// Holds a delegate that is executed when the process has been canceled. */
	FSimpleDelegate CanceledDelegate;

	// Holds a delegate that is executed when a monitored process completed. */
	FOnMonitoredProcessCompleted CompletedDelegate;

	// Holds a delegate that is executed when a monitored process produces output. */
	FOnMonitoredProcessOutput OutputDelegate;
};


class FSerializedUATProcess : public FMonitoredProcess
{
public:
	/**
	 * Get the host-platform-specific path to the UAT running script
	 */
	static CORE_API FString GetUATPath();

public:
	CORE_API FSerializedUATProcess(const FString& RunUATCommandline);

	/**
	 * Run UAT, serially with other FSerializedUATProcess objects. Because the actual call is delayed, this will
	 * always return true, and the LaunchFailedDelegate will be called later if an error happens
	 */
	CORE_API virtual bool Launch() override;

	/**
	 * Returns a delegate that is executed when the process has been canceled.
	 *
	 * @return The delegate.
	 */
	FSimpleDelegate& OnLaunchFailed()
	{
		return LaunchFailedDelegate;
	}


private:

	CORE_API bool LaunchNext();
	CORE_API bool LaunchInternal();
	static CORE_API void CancelQueue();

	// When this one completes, run the next in line
	FSerializedUATProcess* NextProcessToRun = nullptr;

	// Holds a delegate that is executed when the process fails to launch (delayed, in a thread). Used in place of the return value
	// of Launch in the parent class, since it's async
	FSimpleDelegate LaunchFailedDelegate;

	static CORE_API FCriticalSection Serializer;
	static CORE_API bool bHasSucceededOnce;
	static CORE_API FSerializedUATProcess* HeadProcess;
};
