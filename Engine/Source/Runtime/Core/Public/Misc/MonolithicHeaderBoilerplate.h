// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#if UE_IS_ENGINE_MODULE && !defined(SUPPRESS_MONOLITHIC_HEADER_WARNINGS) && !defined(UE_DIRECT_HEADER_COMPILE)
	#define MONOLITHIC_HEADER_BOILERPLATE() COMPILE_WARNING("Monolithic headers should not be used by this module. Please change it to explicitly include the headers it needs.")
	#define SUPPRESS_MONOLITHIC_HEADER_WARNINGS 1
#else
	#undef MONOLITHIC_HEADER_BOILERPLATE
	#define MONOLITHIC_HEADER_BOILERPLATE()
#endif