// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformMath.h"
#include <limits>

#define UE_API CORE_API

namespace UE
{

/**
 * A span of time measured in seconds between two time points.
 *
 * @see FMonotonicTimePoint
 */
struct FMonotonicTimeSpan
{
public:
	constexpr FMonotonicTimeSpan() = default;

	constexpr static FMonotonicTimeSpan Zero() { return FMonotonicTimeSpan(); }

	constexpr static FMonotonicTimeSpan Infinity() { return FromSeconds(std::numeric_limits<double>::infinity()); }

	constexpr static FMonotonicTimeSpan FromSeconds(double Seconds)
	{
		FMonotonicTimeSpan TimeSpan;
		TimeSpan.Time = Seconds;
		return TimeSpan;
	}

	constexpr static FMonotonicTimeSpan FromMilliseconds(double Milliseconds) { return FromSeconds(Milliseconds * 0.001); }

	constexpr double ToSeconds() const { return Time; }
	constexpr double ToMilliseconds() const { return Time * 1000.0; }

	constexpr bool IsZero() const { return *this == Zero(); }
	constexpr bool IsInfinity() const { return *this == Infinity(); }
	bool IsNaN() const { return FPlatformMath::IsNaN(Time); }

	constexpr bool operator==(const FMonotonicTimeSpan Other) const { return Time == Other.Time; }
	constexpr bool operator!=(const FMonotonicTimeSpan Other) const { return Time != Other.Time; }
	constexpr bool operator<=(const FMonotonicTimeSpan Other) const { return Time <= Other.Time; }
	constexpr bool operator< (const FMonotonicTimeSpan Other) const { return Time <  Other.Time; }
	constexpr bool operator>=(const FMonotonicTimeSpan Other) const { return Time >= Other.Time; }
	constexpr bool operator> (const FMonotonicTimeSpan Other) const { return Time >  Other.Time; }

	constexpr FMonotonicTimeSpan operator+() const { return *this; }
	constexpr FMonotonicTimeSpan operator-() const { return FromSeconds(-Time); }

	constexpr FMonotonicTimeSpan operator+(const FMonotonicTimeSpan Span) const { return FromSeconds(Time + Span.Time); }
	constexpr FMonotonicTimeSpan operator-(const FMonotonicTimeSpan Span) const { return FromSeconds(Time - Span.Time); }

	constexpr FMonotonicTimeSpan& operator+=(const FMonotonicTimeSpan Span) { return *this = *this + Span; }
	constexpr FMonotonicTimeSpan& operator-=(const FMonotonicTimeSpan Span) { return *this = *this - Span; }

private:
	double Time = 0;
};

/**
 * A point in time measured in seconds since an arbitrary epoch.
 *
 * This is a monotonic clock which means the current time will never decrease. This time is meant
 * primarily for measuring intervals. The interval between ticks of this clock is constant except
 * for the time that the system is suspended on certain platforms. The tick frequency will differ
 * between platforms, and must not be used as a means of communicating time without communicating
 * the tick frequency together with the time.
 */
struct FMonotonicTimePoint
{
public:
	UE_API static FMonotonicTimePoint Now();

	constexpr FMonotonicTimePoint() = default;

	constexpr static FMonotonicTimePoint Infinity() { return FromSeconds(std::numeric_limits<double>::infinity()); }

	/** Construct from seconds since the epoch. */
	constexpr static FMonotonicTimePoint FromSeconds(const double Seconds)
	{
		FMonotonicTimePoint TimePoint;
		TimePoint.Time = Seconds;
		return TimePoint;
	}

	/** Seconds since the epoch. */
	constexpr double ToSeconds() const { return Time; }

	constexpr bool IsInfinity() const { return *this == Infinity(); }
	bool IsNaN() const { return FPlatformMath::IsNaN(Time); }

	constexpr bool operator==(const FMonotonicTimePoint Other) const { return Time == Other.Time; }
	constexpr bool operator!=(const FMonotonicTimePoint Other) const { return Time != Other.Time; }
	constexpr bool operator<=(const FMonotonicTimePoint Other) const { return Time <= Other.Time; }
	constexpr bool operator< (const FMonotonicTimePoint Other) const { return Time <  Other.Time; }
	constexpr bool operator>=(const FMonotonicTimePoint Other) const { return Time >= Other.Time; }
	constexpr bool operator> (const FMonotonicTimePoint Other) const { return Time >  Other.Time; }

	constexpr FMonotonicTimePoint operator+(const FMonotonicTimeSpan Span) const { return FromSeconds(Time + Span.ToSeconds()); }
	constexpr FMonotonicTimePoint operator-(const FMonotonicTimeSpan Span) const { return FromSeconds(Time - Span.ToSeconds()); }

	constexpr FMonotonicTimeSpan operator-(const FMonotonicTimePoint Point) const
	{
		return FMonotonicTimeSpan::FromSeconds(Time - Point.Time);
	}

private:
	double Time = 0;
};

} // namespace UE

#undef UE_API
