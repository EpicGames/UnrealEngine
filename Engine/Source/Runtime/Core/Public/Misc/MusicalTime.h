// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Serialization/Archive.h"

namespace MusicalTime
{
	static constexpr int32 TicksPerQuarterNote = 960;
}

struct FMusicalTime
{
	FMusicalTime()
		: Bar(0)
		, TickInBar(0)
		, TicksPerBar(0)
		, TicksPerBeat(0)
	{}

	FMusicalTime(int32 InBar, int32 InTickInBar = 0, int32 InTicksPerBar = MusicalTime::TicksPerQuarterNote * 4, int32 InTicksPerBeat = MusicalTime::TicksPerQuarterNote)
		: Bar(InBar)
		, TickInBar(InTickInBar)
		, TicksPerBar(InTicksPerBar)
		, TicksPerBeat(InTicksPerBeat)
	{}

	/** IMPORTANT: If you change the struct data, ensure that you also update the version in NoExportTypes.h  */

	int32 Bar;
	int32 TickInBar;
	int32 TicksPerBar;
	int32 TicksPerBeat;

	/**
	 * Verify that this musical time is valid to use
	 */
	bool IsValid() const
	{
		return TicksPerBar > 0 && TicksPerBeat > 0;
	}

	double FractionalBeatInBar() const
	{
		if (!IsValid())
		{
			return 0.0;
		}
		return static_cast<double>(TickInBar) / static_cast<double>(TicksPerBeat);
	}

	double FractionalBar() const
	{
		if (!IsValid())
		{
			return 0.0;
		}
		return static_cast<double>(Bar) + (static_cast<double>(TickInBar) / static_cast<double>(TicksPerBar));
	}

	/**
	 * Serializes the given musical time from or into the specified archive
	 */
	bool Serialize(FArchive& Ar);

	/**
	 *  return fractional bars between two musical times
	 */
	double operator-(const FMusicalTime& Other)
	{
		return FractionalBar() - Other.FractionalBar();
	}

	static FMusicalTime FloorBar(const FMusicalTime& InMusicalTime)
	{
		FMusicalTime OutMusicalTime = InMusicalTime;
		OutMusicalTime.TickInBar = 0;
		return OutMusicalTime;
	}

	static FMusicalTime FloorBeat(const FMusicalTime& InMusicalTime)
	{
		FMusicalTime OutMusicalTime = InMusicalTime;
		int32 WholeBeatInBar = OutMusicalTime.TickInBar / OutMusicalTime.TicksPerBeat;
		OutMusicalTime.TickInBar = WholeBeatInBar * OutMusicalTime.TicksPerBeat;
		return OutMusicalTime;
	}
	
public:

	friend FArchive& operator<<(FArchive& Ar, FMusicalTime& MusicalTime)
	{
		MusicalTime.Serialize(Ar);
		return Ar;
	}

	friend UE_FORCEINLINE_HINT bool operator==(const FMusicalTime& A, const FMusicalTime& B)
	{
		return A.Bar == B.Bar && A.TickInBar == B.TickInBar;
	}


	friend UE_FORCEINLINE_HINT bool operator!=(const FMusicalTime& A, const FMusicalTime& B)
	{
		return A.Bar != B.Bar || A.TickInBar != B.TickInBar;
	}


	friend UE_FORCEINLINE_HINT bool operator> (const FMusicalTime& A, const FMusicalTime& B)
	{
		return A.Bar >  B.Bar || ( A.Bar == B.Bar && A.TickInBar > B.TickInBar );
	}


	friend UE_FORCEINLINE_HINT bool operator>=(const FMusicalTime& A, const FMusicalTime& B)
	{
		return A.Bar > B.Bar || ( A.Bar == B.Bar && A.TickInBar >= B.TickInBar );
	}


	friend UE_FORCEINLINE_HINT bool operator< (const FMusicalTime& A, const FMusicalTime& B)
	{
		return A.Bar <  B.Bar || ( A.Bar == B.Bar && A.TickInBar < B.TickInBar );
	}


	friend UE_FORCEINLINE_HINT bool operator<=(const FMusicalTime& A, const FMusicalTime& B)
	{
		return A.Bar < B.Bar || ( A.Bar == B.Bar && A.TickInBar <= B.TickInBar );
	}
};

inline bool FMusicalTime::Serialize(FArchive& Ar)
{
	Ar << Bar;
	Ar << TickInBar;
	Ar << TicksPerBar;
	Ar << TicksPerBeat;
	return true;
}
