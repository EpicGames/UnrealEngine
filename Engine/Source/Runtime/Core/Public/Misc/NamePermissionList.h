// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/DirectoryTree.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FString;

/** List of owner names that requested a specific item filtered, allows unregistering specific set of changes by a given plugin or system */
typedef TArray<FName> FPermissionListOwners;

class FNamePermissionList : public TSharedFromThis<FNamePermissionList>
{
public:
	FNamePermissionList() {}
	virtual ~FNamePermissionList() = default;

	/** Returns true if passes filter restrictions using exact match */
	CORE_API bool PassesFilter(const FName Item) const;

	/** 
	 * Add item to DenyList, this specific item will be filtered out.
	 * @return whether the filters changed.
	 */
	CORE_API bool AddDenyListItem(const FName OwnerName, const FName Item);

	/**
	 * Add item to allowlist after which all items not in the allowlist will be filtered out.
	 * @return whether the filters changed.
	 */
	CORE_API bool AddAllowListItem(const FName OwnerName, const FName Item);

	/**
	 * Removes a previously-added item from the DenyList.
	 * @return whether the filters changed.
	 */
	CORE_API bool RemoveDenyListItem(const FName OwnerName, const FName Item);

	/**
	 * Removes a previously-added item from the allowlist.
	 * @return whether the filters changed.
	 */
	CORE_API bool RemoveAllowListItem(const FName OwnerName, const FName Item);

	/**
	 * Set to filter out all items.
	 * @return whether the filters changed.
	 */
	CORE_API bool AddDenyListAll(const FName OwnerName);
	
	/** True if has filters active */
	CORE_API bool HasFiltering() const;

	/** Gathers the names of all the owners in this DenyList. */
	CORE_API TArray<FName> GetOwnerNames() const;

	/** 
	* Removes all filtering changes associated with a specific owner name.
	 * @return whether the filters changed.
	 */
	CORE_API bool UnregisterOwner(const FName OwnerName);

	/**
	 * Removes all filtering changes associated with the specified list of owner names.
	 * @return whether the filters changed.
	 */
	CORE_API bool UnregisterOwners(const TArray<FName>& OwnerNames);

	/**
	 * Add the specified filters to this one.
	 * @return whether the filters changed.
	 */
	CORE_API bool Append(const FNamePermissionList& Other);

	/**
	* Unregisters specified owners then adds specified filters in one operation (to avoid multiple filters changed events).
	* @return whether the filters changed.
	*/
	CORE_API bool UnregisterOwnersAndAppend(const TArray<FName>& OwnerNamesToRemove, const FNamePermissionList& FiltersToAdd);

	/** Get raw DenyList */
	const TMap<FName, FPermissionListOwners>& GetDenyList() const { return DenyList; }
	
	/** Get raw allowlist */
	const TMap<FName, FPermissionListOwners>& GetAllowList() const { return AllowList; }

	/** Are all items set to be filtered out */
	bool IsDenyListAll() const { return DenyListAll.Num() > 0; }

	/** Triggered when filter changes */
	FSimpleMulticastDelegate& OnFilterChanged() { return OnFilterChangedDelegate; }

protected:

	/** List if items to filter out */
	TMap<FName, FPermissionListOwners> DenyList;

	/** List of items to allow, if not empty all items will be filtered out unless they are in the list */
	TMap<FName, FPermissionListOwners> AllowList;

	/** List of owner names that requested all items to be filtered out */
	FPermissionListOwners DenyListAll;

	/** Triggered when filter changes */
	FSimpleMulticastDelegate OnFilterChangedDelegate;

	/** Temporarily prevent delegate from being triggered */
	bool bSuppressOnFilterChanged = false;
};

enum class EPathPermissionListType
{
	Default,	// Default path permission list
	ClassPaths	// Class permission list
};

// Result of non-exact filtering on path prefixes
enum class EPathPermissionPrefixResult
{
	// The query against the list failed because of one of the following:
	// 	* There was an explicit allow-list and none of the entries was a parent path of the query path
	//		(or a child of the query path when bAllowParentPaths = true)
	//	* There were deny-list entries and one of the entries was a parent path of the query path
	Fail,
	// The query against the list failed and either:
	//	* All paths are denied, so queries for child paths will also fail
	//	* The query failed on a deny-list entry, so child paths will fail on this same entry
	//	* There is an explicit allow-list and there are no entries with more components than the query path,
	//	  so children can never pass.
	FailRecursive,
	// The query against the list succeeded, but queries for child paths may fail because there are longer
	// paths in the deny-list with more components
	Pass,
	// The query against the list succeeded and queries for child paths will all succeed no deny-list entry can
	// possibly fail them
	PassRecursive,
};

DECLARE_DELEGATE_RetVal(bool, FDelegateEnablePathFilter);

/**
 * Set of paths that are allowd and/or denied for certain use cases.
 * A permission list may contain
 * 	- Blanket denial
 * 	- Specifically denied paths
 * 	- Specifically allowed paths
 * In decreasing order of priority. When performing prefix checks, if a a path matches a denied path, it cannot be
 * allowed again by a more specific allowed path. If any paths are specifically allowed, paths which do NOT match
 * something in the allow list are implicitly denied.
 */
class FPathPermissionList : public TSharedFromThis<FPathPermissionList>
{
public:
	CORE_API FPathPermissionList(EPathPermissionListType InType = EPathPermissionListType::Default);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FPathPermissionList() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FPathPermissionList(const FPathPermissionList&) = default;
	FPathPermissionList& operator=(const FPathPermissionList&) = default;

	FPathPermissionList(FPathPermissionList&&) = default;
	FPathPermissionList& operator=(FPathPermissionList&&) = default;

	/** Returns true if passes filter restrictions using exact match */
	CORE_API bool PassesFilter(const FStringView Item) const;

	/** Returns true if passes filter restrictions using exact match */
	CORE_API bool PassesFilter(const FName Item) const;

	/** Returns true if passes filter restrictions using exact match */
	CORE_API bool PassesFilter(const TCHAR* Item) const;

	/** Returns true if passes filter restrictions for path */
	CORE_API bool PassesStartsWithFilter(const FStringView Item, const bool bAllowParentPaths = false) const;

	/** Returns true if passes filter restrictions for path */
	CORE_API bool PassesStartsWithFilter(const FName Item, const bool bAllowParentPaths = false) const;

	/** Returns true if passes filter restrictions for path */
	CORE_API bool PassesStartsWithFilter(const TCHAR* Item, const bool bAllowParentPaths = false) const;

	/**
	 * Checks the given path against the restrictions and return whether it's possible for any child paths to succeed or
	 * fail as well. Returning PassRecursive or FailRecursive guarantees that no child paths of the queried path can
	 * fail or pass the filter respectively. Returning Pass or Fail does not guarantee that there is some path which
	 * fails or passes the filter respectively.
	 *
	 * Examples:
	 *
	 * Given no deny or allow lists:
	 * Inputs:
	 * 	/ -> PassRecursive, because no paths can fail to match the allow list or match the deny list.
	 *
	 * Given a deny-list entry:
	 *  Allow: empty
	 *  Deny: /Secret
	 * Inputs:
	 * 	/ -> Pass, because some children of this path may be denied.
	 *  /Secret -> FailRecursive, because this path is denied and all children will also be denied.
	 *  /Public -> PassRecursive, because this path is not demied and no children can be denied.
	 *
	 * Given an allow-list entry:
	 * 	Allow: /JustThis
	 *  Deny: empty
	 * Inputs:
	 *  / -> Fail
	 * 	/JustThis -> PassRecursive
	 *  /SomethingElse -> FailRecursive
	 *
	 * Given both allow and deny-lists:
	 *  Allow: /Stuff
	 * 	Deny: /Stuff/Secret
	 * Inputs:
	 * 	/ -> Fail
	 *  /Stuff -> Pass
	 * 	/Stuff/Secret -> Fail
	 * 	/Stuff/Public -> PassRecursive
	 */
	CORE_API EPathPermissionPrefixResult PassesStartsWithFilterRecursive(const FStringView Item,
		const bool bAllowParentPaths = false) const;

	/** 
	 * Add item to DenyList, this specific item will be filtered out.
	 * @return whether the filters changed.
	 */
	CORE_API bool AddDenyListItem(const FName OwnerName, const FStringView Item);

	/**
	 * Add item to DenyList, this specific item will be filtered out.
	 * @return whether the filters changed.
	 */
	CORE_API bool AddDenyListItem(const FName OwnerName, const FName Item);

	/**
	 * Add item to DenyList, this specific item will be filtered out.
	 * @return whether the filters changed.
	 */
	CORE_API bool AddDenyListItem(const FName OwnerName, const TCHAR* Item);

	/** Returns whether the given path has been denied explicitly with a call to AddDenyListItem. */
	CORE_API bool ContainsDenyListItem(FStringView Item) const;

	/** Returns whether this list has any explicitly denied paths. */
	CORE_API bool HasDenyListEntries() const;

	/** Get a copy of the paths explicity denied in this list. */
	CORE_API TArray<FString> GetDenyListEntries() const;

	/**
	* Remove item from the DenyList
	* @return whether the filters changed
	*/
	CORE_API bool RemoveDenyListItem(const FName OwnerName, const FStringView Item);

	/**
	* Remove item from the DenyList
	* @return whether the filters changed
	*/
	CORE_API bool RemoveDenyListItem(const FName OwnerName, const FName Item);

	/**
	* Remove item from the DenyList
	* @return whether the filters changed
	*/
	CORE_API bool RemoveDenyListItem(const FName OwnerName, const TCHAR* Item);

	/**
	 * Removes an item from the deny list and returns a list of all the owners of that item
	 * so that the item can be re-introduced.
	 */
	CORE_API FPermissionListOwners RemoveDenyListItemAndGetOwners(FStringView Item);

	/**
	 * Add item to allowlist after which all items not in the allowlist will be filtered out.
	 * @return whether the filters changed.
	 */
	CORE_API bool AddAllowListItem(const FName OwnerName, const FStringView Item);

	/**
	 * Add item to allowlist after which all items not in the allowlist will be filtered out.
	 * @return whether the filters changed.
	 */
	CORE_API bool AddAllowListItem(const FName OwnerName, const FName Item);

	/**
	 * Add item to allowlist after which all items not in the allowlist will be filtered out.
	 * @return whether the filters changed.
	 */
	CORE_API bool AddAllowListItem(const FName OwnerName, const TCHAR* Item);

	/** Returns whether this list has any explicitly allowed paths, which will lead to it denying access to any paths
	 * not listed.*/
	CORE_API bool HasAllowListEntries() const;

	/** Returns a copy of the paths explicity allowed in this list */
	CORE_API TArray<FString> GetAllowListEntries() const;

	/**
	* Remove item from the AllowList
	* @return whether the filters changed
	*/
	CORE_API bool RemoveAllowListItem(const FName OwnerName, const FStringView Item);

	/**
	* Remove item from the AllowList
	* @return whether the filters changed
	*/
	CORE_API bool RemoveAllowListItem(const FName OwnerName, const FName Item);

	/**
	* Remove item from the AllowList
	* @return whether the filters changed
	*/
	CORE_API bool RemoveAllowListItem(const FName OwnerName, const TCHAR* Item);

	/**
	 * Set to filter out all items.
	 * @return whether the filters changed.
	 */
	CORE_API bool AddDenyListAll(const FName OwnerName);
	
	/** True if has filters active */
	CORE_API bool HasFiltering() const;

	/** Gathers the names of all the owners in this DenyList. */
	CORE_API TArray<FName> GetOwnerNames() const;

	/**
	 * Removes all filtering changes associated with a specific owner name.
	 * @return whether the filters changed.
	 */
	CORE_API bool UnregisterOwner(const FName OwnerName);

	/**
	 * Removes all filtering changes associated with the specified list of owner names.
	 * @return whether the filters changed.
	 */
	CORE_API bool UnregisterOwners(const TArray<FName>& OwnerNames);
	
	/**
	 * Add the specified filters to this one. Rules are not applied, direct append lists.
	 * @return whether the filters changed.
	 */
	CORE_API bool Append(const FPathPermissionList& Other);

	/**
	 * Combine two filters.
	 * Result will contain all DenyList paths combined.
	 * Result will contain AllowList paths that pass both filters.
	 * @return new combined filter.
	 */
	[[nodiscard]] CORE_API FPathPermissionList CombinePathFilters(const FPathPermissionList& OtherFilter) const;

	/**
	* Unregisters specified owners then adds specified filters in one operation (to avoid multiple filters changed events).
	* @return whether the filters changed.
	*/
	CORE_API bool UnregisterOwnersAndAppend(const TArray<FName>& OwnerNamesToRemove, const FPathPermissionList& FiltersToAdd);

	/** Are all items set to be filtered out */
	bool IsDenyListAll() const { return DenyListAll.Num() > 0; }
	
	/** Triggered when filter changes */
	FSimpleMulticastDelegate& OnFilterChanged() { return OnFilterChangedDelegate; }

	CORE_API void AddEnablePathFilterDelegate(FName Owner, FDelegateEnablePathFilter Delegate);
	CORE_API bool RemoveEnablePathFilterDelegate(FName Owner);

	/** Dumps the path permission list details into a multi-line string */
	CORE_API FString ToString() const;

protected:

	/**
	 * Checks if an item is of a valid format for this list
	 * @return True if the item passes list type test.
	 */
	CORE_API void VerifyItemMatchesListType(const FStringView Item) const;

	/** Compiled path tree produced from DenyList */
	TDirectoryTree<FPermissionListOwners> DenyTree;

	/** Compiled path tree produced from AllowList */
	TDirectoryTree<FPermissionListOwners> AllowTree;

	/** List of owner names that requested all items to be filtered out */
	FPermissionListOwners DenyListAll;
	
	/** Triggered when filter changes */
	FSimpleMulticastDelegate OnFilterChangedDelegate;

	FDelegateEnablePathFilter OnEnablePathFilterDelegate;

	/** Temporarily prevent delegate from being triggered */
	bool bSuppressOnFilterChanged = false;

	/** Type of paths this list represent */
	EPathPermissionListType ListType;

	TMap<FName, FDelegateEnablePathFilter> EnablePathFilterDelegates;
};
