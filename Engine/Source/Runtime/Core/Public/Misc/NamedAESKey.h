// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "Misc/AES.h"
#include "Misc/Guid.h"

struct FNamedAESKey
{
	FString Name;
	FGuid Guid;
	FAES::FAESKey Key;

	bool IsValid() const
	{
		return Key.IsValid();
	}
};
