// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"

/**
 * Implements a globally unique identifier for network related use.
 */
class FNetworkGUID
{
public:

	union
	{
		uint64 ObjectId;
	};

	FNetworkGUID()
		: ObjectId(0)
	{ }
		
	friend bool operator==(const FNetworkGUID& X, const FNetworkGUID& Y)
	{
		return (X.ObjectId == Y.ObjectId);
	}

	friend bool operator!=(const FNetworkGUID& X, const FNetworkGUID& Y)
	{
		return (X.ObjectId != Y.ObjectId);
	}
	
	friend FArchive& operator<<(FArchive& Ar, FNetworkGUID& G)
	{
		Ar.SerializeIntPacked64(G.ObjectId);
		return Ar;
	}

	friend uint32 GetTypeHash(const FNetworkGUID& Guid)
	{
		return ::GetTypeHash(Guid.ObjectId);
	}

	bool IsDynamic() const
	{
		return IsValid() && !IsStatic();
	}

	bool IsStatic() const
	{
		return ObjectId & 1;
	}

	bool IsValid() const
	{
		return ObjectId > 0;
	}

	/** A Valid but unassigned NetGUID */
	bool IsDefault() const
	{
		return (ObjectId == 1);
	}

	static FNetworkGUID GetDefault()
	{
		return CreateFromIndex(0, true);
	}

	void Reset()
	{
		ObjectId = 0;
	}

	FString ToString() const
	{
		return FString::Printf(TEXT("%" UINT64_FMT), ObjectId);
	}

	bool operator<(const FNetworkGUID& Other) const
	{
		return ObjectId < Other.ObjectId;
	}

	static FNetworkGUID CreateFromIndex(uint64 NetIndex, bool bIsStatic)
	{
		check(NetIndex <= (MAX_uint64 >> 1));

		FNetworkGUID NewGuid;
		NewGuid.ObjectId = NetIndex << 1 | (bIsStatic ? 1 : 0);

		return NewGuid;
	}
};
