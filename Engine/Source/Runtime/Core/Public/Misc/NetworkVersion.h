// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Logging/LogMacros.h"
#include "Misc/EngineNetworkCustomVersion.h"

// The version number used for determining network compatibility. If zero, uses the engine compatible version.
#define ENGINE_NET_VERSION  0

// The version number used for determining replay compatibility
#define ENGINE_REPLAY_VERSION  ENGINE_NET_VERSION

CORE_API DECLARE_LOG_CATEGORY_EXTERN( LogNetVersion, Log, All );

class FNetworkReplayVersion
{
public:
	FNetworkReplayVersion() 
		: NetworkVersion(0)
		, Changelist(0)
	{
	}

	FNetworkReplayVersion(const FString& InAppString, const uint32 InNetworkVersion, const uint32 InChangelist) 
		: AppString(InAppString)
		, NetworkVersion(InNetworkVersion)
		, Changelist(InChangelist)
	{
	}

	FString		AppString;
	/** This is a hash of compatible versions, and not pulled directly from any version enums. */
	uint32		NetworkVersion;
	uint32		Changelist;
};

/**
 * List of runtime features that can affect network compatibility between two connections
 */
enum class EEngineNetworkRuntimeFeatures : uint16
{
	None = 0,
	IrisEnabled = 1 << None, // Are we running the Iris or Generic replication system
	UseRemoteObjectReferences = 1 << IrisEnabled, // Are UObjects being serialized as remote object references
};
ENUM_CLASS_FLAGS(EEngineNetworkRuntimeFeatures);

struct FNetworkVersion
{
	/** Called in GetLocalNetworkVersion if bound */
	DECLARE_DELEGATE_RetVal( uint32, FGetLocalNetworkVersionOverride );
	static CORE_API FGetLocalNetworkVersionOverride GetLocalNetworkVersionOverride;

	/** Called in IsNetworkCompatible if bound */
	DECLARE_DELEGATE_RetVal_TwoParams( bool, FIsNetworkCompatibleOverride, uint32, uint32 );
	static CORE_API FIsNetworkCompatibleOverride IsNetworkCompatibleOverride;

	/** Called in GetReplayCompatibleChangelist if bound */
	DECLARE_DELEGATE_RetVal(uint32, FGetReplayCompatibleChangeListOverride);
	static CORE_API FGetReplayCompatibleChangeListOverride GetReplayCompatibleChangeListOverride;

	static CORE_API uint32 GetNetworkCompatibleChangelist();
	static CORE_API uint32 GetReplayCompatibleChangelist();

	static CORE_API uint32 GetNetworkProtocolVersion(const FGuid& VersionGuid);
	static CORE_API uint32 GetCompatibleNetworkProtocolVersion(const FGuid& VersionGuid);

	static CORE_API const FCustomVersionContainer& GetNetworkCustomVersions();

	static CORE_API void RegisterNetworkCustomVersion(const FGuid& VersionGuid, int32 Version, int32 CompatibleVersion, const FName& FriendlyName);

	/**
	* Generates a version number, that by default, is based on a checksum of the engine version + project name + project version string
	* Game/project code can completely override what this value returns through the GetLocalNetworkVersionOverride delegate
	* If called with AllowOverrideDelegate=false, we will not call the game project override. (This allows projects to call base implementation in their project implementation)
	*/
	static CORE_API uint32 GetLocalNetworkVersion( bool AllowOverrideDelegate=true );

	/**
	* Determine if a connection is compatible with this instance
	*
	* @param bRequireEngineVersionMatch should the engine versions match exactly
	* @param LocalNetworkVersion current version of the local machine
	* @param RemoteNetworkVersion current version of the remote machine
	*
	* @return true if the two instances can communicate, false otherwise
	*/
	static CORE_API bool IsNetworkCompatible( const uint32 LocalNetworkVersion, const uint32 RemoteNetworkVersion );

	/**
	* Generates a special struct that contains information to send to replay server
	*/
	static CORE_API FNetworkReplayVersion GetReplayVersion();

	/**
	* Sets the project version used for networking. Needs to be a function to verify
	* string and correctly invalidate cached values
	* 
	* @param  InVersion
	* @return void
	*/
	static CORE_API void SetProjectVersion(const TCHAR* InVersion);

	/**
	* Sets the game network protocol version used for networking and invalidate cached values
	*/
	static CORE_API void SetGameNetworkProtocolVersion(uint32 GameNetworkProtocolVersion);

	/**
	* Sets the game compatible network protocol version used for networking and invalidate cached values
	*/
	static CORE_API void SetGameCompatibleNetworkProtocolVersion(uint32 GameCompatibleNetworkProtocolVersion);

	/**
	 * Compares if the connection's runtime features are compatible with each other
	 */
	static CORE_API bool AreNetworkRuntimeFeaturesCompatible(EEngineNetworkRuntimeFeatures LocalFeatures, EEngineNetworkRuntimeFeatures RemoteFeatures);

	/**
	 * Build and return a string describing the status of the the network runtime features bitflag
	 */
	static CORE_API void DescribeNetworkRuntimeFeaturesBitset(EEngineNetworkRuntimeFeatures FeaturesBitflag, FStringBuilderBase& OutVerboseDescription);
	
	/**
	* Returns the project version used by networking
	* 
	* @return FString
	*/
	static const FString& GetProjectVersion() { return GetProjectVersion_Internal(); }

	/**
	* Invalidates any cached network checksum and forces it to be recalculated on next request
	*/
	static void InvalidateNetworkChecksum() { bHasCachedNetworkChecksum = false; }

protected:

	/**
	* Used to allow BP only projects to override network versions
	*/
	static CORE_API FString& GetProjectVersion_Internal();

	static CORE_API bool		bHasCachedNetworkChecksum;
	static CORE_API uint32	CachedNetworkChecksum;

	static CORE_API bool		bHasCachedReplayChecksum;
	static CORE_API uint32	CachedReplayChecksum;
};
