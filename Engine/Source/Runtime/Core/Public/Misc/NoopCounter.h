// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/** Fake Thread safe counter, used to avoid cluttering code with ifdefs when counters are only used for debugging. */
class FNoopCounter
{
public:
	using IntegerType = int32;

	FNoopCounter() { }
	FNoopCounter( const FNoopCounter& Other ) { }
	FNoopCounter( int32 Value ) { }

	int32 Increment()
	{
		return 0;
	}

	int32 Add( int32 Amount )
	{
		return 0;
	}

	int32 Decrement()
	{
		return 0;
	}

	int32 Subtract( int32 Amount )
	{
		return 0;
	}

	int32 Set( int32 Value )
	{
		return 0;
	}

	int32 Reset()
	{
		return 0;
	}

	int32 GetValue() const
	{
		return 0;
	}
};
