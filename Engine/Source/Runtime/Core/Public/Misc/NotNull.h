// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/IntrusiveUnsetOptionalState.h"
#include "Misc/UEOps.h"
#include "Templates/Requires.h"
#include "Templates/TypeHash.h"
#include "Traits/IsImplicitlyConstructible.h"

#include <type_traits>

/**
 * TNotNull is a wrapper template which is used to indicate that a pointer or handle is never intended to be null.
 *
 * Design and rationale:
 *
 * - No default construction, construction with nullptr or comparison against nullptr. Attempting to do so
 *   will cause a compile error.
 *
 * - While intended for pointer types, it should be usable for anything else that is nullable, e.g. TFunction.
 *
 * - The pointer value is checked on construction or assignment, and then never after that. The only exception
 *   to this is a TNotNull in a moved-from state, which will not re-check the value. - see below.
 *
 * - TNotNull is movable.  This allows things like TNotNull<TUniquePtr<T>> to be expressed, but means that a
 *   moved-from TNotNull variable can end up being null. However, users are never required to handle a variable
 *   in this state. If a variable is to be reused after it has been moved from (e.g. a data member of an object
 *   that isn't being destroyed) then code that made it null should assign a new non-null value to it before
 *   returning to user code or using it to construct or assign to another TNotNull. Users of your variable should
 *   never be allowed to see a null value. Compilers and static analyzers are allowed to assume that the pointer
     is not null and optimize and analyze accordingly.
 *
 * - The UE_ENABLE_NOTNULL_WRAPPER can be used to disable all checking and become an alias to the inner type, so
 *   there is no runtime overhead at all.
 */

#ifndef UE_ENABLE_NOTNULL_WRAPPER
	#define UE_ENABLE_NOTNULL_WRAPPER (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)
#endif

#ifdef __clang__
	// Disable clang warnings about returns_nonnull only being applicable to raw pointers
	#define UE_NOTNULL_FUNCTION_NON_NULL_RETURN_START _Pragma("clang diagnostic push") _Pragma("clang diagnostic ignored \"-Wignored-attributes\"") FUNCTION_NON_NULL_RETURN_START _Pragma("clang diagnostic pop")
	#define UE_NOTNULL_FUNCTION_NON_NULL_RETURN_END   _Pragma("clang diagnostic push") _Pragma("clang diagnostic ignored \"-Wignored-attributes\"") FUNCTION_NON_NULL_RETURN_END   _Pragma("clang diagnostic pop")
#else
	#define UE_NOTNULL_FUNCTION_NON_NULL_RETURN_START FUNCTION_NON_NULL_RETURN_START
	#define UE_NOTNULL_FUNCTION_NON_NULL_RETURN_END   FUNCTION_NON_NULL_RETURN_END
#endif

#if UE_ENABLE_NOTNULL_WRAPPER

	class FArchive;

	template <typename T>
	struct TNotNull;

	namespace UE::Core::Private
	{
#if DO_CHECK
		template <typename T, typename = void>
		struct TSupportsNotOperator
		{
			static constexpr bool Value = false;
		};

		template <typename T>
		struct TSupportsNotOperator<T, std::void_t<decltype(!std::declval<T>())>>
		{
			static constexpr bool Value = true;
		};

		template <typename T>
		inline constexpr bool TSupportsNotOperator_V = TSupportsNotOperator<T>::Value;

		[[noreturn]] CA_NO_RETURN CORE_API void ReportNotNullPtr();
		CORE_API void CheckLoadingNotNullPtr(FArchive& Ar);
#endif // DO_CHECK

		template <typename... ArgTypes>
		struct TIsTNotNullParam
		{
			static constexpr bool Value = false;
		};

		template <typename T>
		struct TIsTNotNullParam<TNotNull<T>>
		{
			static constexpr bool Value = true;
		};

		template <typename... ArgTypes>
		inline constexpr bool TIsTNotNullParam_V = TIsTNotNullParam<ArgTypes...>::Value;

		template <typename T>
		struct TNotNullBase
		{
			using Sub = TNotNull<T>;

			// Allow implicit conversion to the wrapped value, for compatibility with non-TNotNull APIs
			operator T() const&
			{
				return ((Sub*)this)->Val;
			}
			operator T() &&
			{
				return (T&&)((Sub*)this)->Val;
			}

			// Allow dereferencing, if it's supported
			const T& operator->() const
			{
				return ((Sub*)this)->Val;
			}
		};

		template <typename T>
		struct TNotNullBase<T*>
		{
			using Sub = TNotNull<T*>;

			// Allow implicit conversion to the wrapped value, for compatibility with non-TNotNull APIs
		UE_NOTNULL_FUNCTION_NON_NULL_RETURN_START
			operator T*() const
		UE_NOTNULL_FUNCTION_NON_NULL_RETURN_END
			{
				return ((Sub*)this)->Val;
			}

			// Allow dereferencing, if it's supported
		UE_NOTNULL_FUNCTION_NON_NULL_RETURN_START
			T* operator->() const
		UE_NOTNULL_FUNCTION_NON_NULL_RETURN_END
			{
				return ((Sub*)this)->Val;
			}
		};
	}

	template <typename T>
	struct TNotNull : UE::Core::Private::TNotNullBase<T>
	{
	private:
		using Super = UE::Core::Private::TNotNullBase<T>;
		friend Super;

		T Val;

	public:
		template <typename>
		friend struct TNotNull;

		// Prevent default construction and construction/assignment against nullptr
		TNotNull() = delete;
		TNotNull(TYPE_OF_NULLPTR) = delete;
		TNotNull& operator=(TYPE_OF_NULLPTR) = delete;
		~TNotNull() = default;

		// Allow direct construction of the inner value
		template <typename... ArgTypes>
			requires (std::is_constructible_v<T, ArgTypes...> && !UE::Core::Private::TIsTNotNullParam_V<std::decay_t<ArgTypes>...>)
		explicit(!TIsImplicitlyConstructible_V<T, ArgTypes...>) TNotNull(ArgTypes&&... Args)
			: Val((ArgTypes&&)Args...)
		{
#if DO_CHECK
			if constexpr (UE::Core::Private::TSupportsNotOperator_V<T>)
			{
				if (!Val)
				{
					UE::Core::Private::ReportNotNullPtr();
				}
			}
#endif
		}

		/////////////////////////////////////////////////
		// Start - intrusive TOptional<TNotNull> state //
		/////////////////////////////////////////////////
		static constexpr bool bHasIntrusiveUnsetOptionalState = HasIntrusiveUnsetOptionalState<T>();
		using IntrusiveUnsetOptionalStateType = TNotNull;

		UE_NODEBUG [[nodiscard]] TNotNull(FIntrusiveUnsetOptionalState Tag)
			requires(bHasIntrusiveUnsetOptionalState)
			: Val(Tag)
		{
		}
		UE_NODEBUG [[nodiscard]] bool operator==(FIntrusiveUnsetOptionalState Tag) const
			requires(bHasIntrusiveUnsetOptionalState)
		{
			return Val == Tag;
		}
		///////////////////////////////////////////////
		// End - intrusive TOptional<TNotNull> state //
		///////////////////////////////////////////////

		// Allow conversions
		template <
			typename OtherType
			UE_REQUIRES(std::is_convertible_v<const OtherType&, T>)
		>
		TNotNull(const TNotNull<OtherType>& Rhs)
			: Val(Rhs.Val)
		{
		}
		template <
			typename OtherType
			UE_REQUIRES(std::is_convertible_v<OtherType&&, T>)
		>
		TNotNull(TNotNull<OtherType>&& Rhs)
			: Val((OtherType&&)Rhs.Val)
		{
		}

		// Forward copying and moving to the inner type
		TNotNull(TNotNull&&) = default;
		TNotNull(const TNotNull&) = default;
		TNotNull& operator=(TNotNull&&) = default;
		TNotNull& operator=(const TNotNull&) = default;

		// Disallow testing for nullness - implicitly or explicitly
		operator bool() const = delete;

		// Allow dereferencing, if it's supported
		decltype(auto) operator*() const
		{
			return *Val;
		}

		// Allow function call syntax, if it's supported
		template <typename... ArgTypes>
		auto operator()(ArgTypes&&... Args) const -> decltype(this->Val((ArgTypes&&)Args...))
		{
			return Val((ArgTypes&&)Args...);
		}

		// Allow hashing, if it's supported
		friend uint32 GetTypeHash(const TNotNull& NotNull)
		{
			return GetTypeHashHelper(NotNull.Val);
		}

		// Allow comparison, if it's supported
		template <typename U>
		[[nodiscard]] auto UEOpEquals(const TNotNull<U>& Rhs) const -> decltype(Val == Rhs.Val)
		{
			return Val == Rhs.Val;
		}
		template <typename U>
		[[nodiscard]] auto UEOpEquals(const U& Rhs) const -> decltype(Val == Rhs)
		{
			static_assert(!std::is_same_v<U, TYPE_OF_NULLPTR>, "Comparing a TNotNull to nullptr is illegal");
			return Val == Rhs;
		}

		// Disallow comparison against nullptr
		bool UEOpEquals(TYPE_OF_NULLPTR Rhs) const = delete;
	};

	// Allow serialization, if it's supported
	template <typename T>
	auto operator<<(FArchive& Ar, TNotNull<T>& Val) -> decltype(Ar << (T&)Val)
	{
		if constexpr (std::is_void_v<decltype(Ar << (T&)Val)>)
		{
			Ar << (T&)Val;
#if DO_CHECK
			if constexpr (UE::Core::Private::TSupportsNotOperator_V<T>)
			{
				if (!(T&)Val)
				{
					UE::Core::Private::CheckLoadingNotNullPtr(Ar);
				}
			}
#endif
		}
		else
		{
			decltype(auto) Result = Ar << (T&)Val;
#if DO_CHECK
			if constexpr (UE::Core::Private::TSupportsNotOperator_V<T>)
			{
				if (!(T&)Val)
				{
					UE::Core::Private::CheckLoadingNotNullPtr(Ar);
				}
			}
#endif
			return Result;
		}
	}

	// Gets the inner value of a TNotNull without requiring a cast and repeat its inner type.
	template <typename T>
	[[nodiscard]] T NotNullGet(const TNotNull<T>& NotNull)
	{
		return NotNull;
	}
	template <typename T>
	[[nodiscard]] T NotNullGet(TNotNull<T>&& NotNull)
	{
		return (TNotNull<T>&&)NotNull;
	}

#undef UE_NOTNULL_FUNCTION_NON_NULL_RETURN_END
#undef UE_NOTNULL_FUNCTION_NON_NULL_RETURN_START

#else // UE_ENABLE_NOTNULL_WRAPPER

	template <typename T>
	using TNotNull = T;

	// Gets the inner value of a TNotNull without requiring a cast and repeat its inner type.
	template <typename T>
	[[nodiscard]] T&& NotNullGet(T&& NotNull)
	{
		return (T&&)NotNull;
	}

#endif // !UE_ENABLE_NOTNULL_WRAPPER
