// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchive.h"
#include "UObject/NameTypes.h"

struct FImage;
struct FImageView;

/**
 * Thumbnail compression interface for packages.  The engine registers a class that can compress and
 * decompress thumbnails that the package linker uses while loading and saving data.
 */
class FThumbnailCompressionInterface
{
public:

	/**
	 * Compresses an image
	 *
	 * @param	InUncompressedData	The uncompressed image data
	 * @param	InWidth				Width of the image
	 * @param	InHeight			Height of the image
	 * @param	OutCompressedData	[Out] Compressed image data
	 * @return	true if the image was compressed successfully, otherwise false if an error occurred
	 */
	virtual bool CompressImage( const TArray< uint8 >& InUncompressedData, const int32 InWidth, const int32 InHeight, TArray< uint8 >& OutCompressedData ) = 0;

	/**
	 * Decompresses an image
	 *
	 * @param	InCompressedData	The compressed image data
	 * @param	InWidth				Width of the image
	 * @param	InHeight			Height of the image
	 * @param	OutUncompressedData	[Out] Uncompressed image data
	 * @return	true if the image was decompressed successfully, otherwise false if an error occurred
	 */
	virtual bool DecompressImage( const TArray< uint8 >& InCompressedData, const int32 InWidth, const int32 InHeight, TArray< uint8 >& OutUncompressedData ) = 0;

	/** Get name of compressor
	 * 
	 * @return Name of thumbnail compressor
	 */
	virtual FName GetThumbnailCompressorName() const = 0;

	/** Is lossless compression
	 * 
	 * @return true if compression is lossless
	 */
	virtual bool IsLosslessCompression() const = 0;

	/**
	 * Get the mime type of the underlying compressed data
	 */
	virtual FStringView GetMimeType() const = 0;
};


/**
 * Thumbnail image data for an object.
 */
class FObjectThumbnail
{
public:

	/**
	 * Static: Sets the thumbnail compressor to use when loading/saving packages.  The caller is
	 * responsible for the object's lifespan.
	 *
	 * @param	InThumbnailCompressor	A class derived from FThumbnailCompressionInterface.
	 */
	static void SetThumbnailCompressors( FThumbnailCompressionInterface* InPNGThumbnailCompressor, 
		FThumbnailCompressionInterface* InJPEGThumbnailCompressor)
	{
		PNGThumbnailCompressor = InPNGThumbnailCompressor;
		JPEGThumbnailCompressor = InJPEGThumbnailCompressor;
	}

private:

	/** Static: Thumbnail compressor. */
	static CORE_API FThumbnailCompressionInterface* PNGThumbnailCompressor;
	static CORE_API FThumbnailCompressionInterface* JPEGThumbnailCompressor;

public:

	/** Default constructor. */
	CORE_API FObjectThumbnail();

	/** Returns the width of the thumbnail. */
	int32 GetImageWidth() const
	{
		return ImageWidth;
	}

	/** Returns the height of the thumbnail. */
	int32 GetImageHeight() const
	{
		return ImageHeight;
	}

	/** @return		the number of bytes in this thumbnail's compressed image data. */
	int32 GetCompressedDataSize() const
	{
		return CompressedImageData.Num();
	}

	/** Sets the image dimensions. */
	void SetImageSize( int32 InWidth, int32 InHeight )
	{
		ImageWidth = InWidth;
		ImageHeight = InHeight;
	}

	/** Returns true if the thumbnail was loaded from disk and not dynamically generated. */
	bool IsLoadedFromDisk(void) const { return bLoadedFromDisk; }

	/** Returns true if the thumbnail was saved AFTER custom-thumbnails for shared thumbnail asset types was supported. */
	bool IsCreatedAfterCustomThumbsEnabled (void) const { return bCreatedAfterCustomThumbForSharedTypesEnabled; }
	
	/** For newly generated custom thumbnails, mark it as valid in the future. */
	void SetCreatedAfterCustomThumbsEnabled(void) { bCreatedAfterCustomThumbForSharedTypesEnabled = true; }

	/** Returns true if the thumbnail is dirty and needs to be regenerated at some point. */
	bool IsDirty() const
	{
		return bIsDirty;
	}

	/** Marks the thumbnail as dirty. */
	void MarkAsDirty()
	{
		bIsDirty = true;
	}

	/** Access the image data in place (does not decompress). */
	TArray< uint8 >& AccessImageData()
	{
		return ImageData;
	}

	/** Access the image data in place (does not decompress) const version. */
	const TArray< uint8 >& AccessImageData() const
	{
		return ImageData;
	}

	/** Access the compressed image data. */
	TArray< uint8 >& AccessCompressedImageData()
	{
		return CompressedImageData;
	}

	/** Returns true if this is an empty thumbnail. */
	bool IsEmpty() const
	{
		return ImageWidth == 0 || ImageHeight == 0;
	}

	/**
	 * Returns if the thumbnail actually has any valid image data or not.
	 * Note that it is possible for ::IsEmpty to return true and this method
	 * to return false if there was a problem during serialization or the 
	 * thumbnail data has otherwise become corrupted.
	 */
	bool HasValidImageData() const
	{
		return !ImageData.IsEmpty() || !CompressedImageData.IsEmpty();
	}

	/** Returns thumbnail compressor used on current compressed image data. */
	CORE_API FThumbnailCompressionInterface* GetCompressor() const;

	/** Returns thumbnail compressor that would be used on current uncompressed image data. */
	CORE_API FThumbnailCompressionInterface* ChooseNewCompressor() const;

	/** Returns uncompressed image data, decompressing it on demand if needed.
	
		Prefer GetImage and use FImage/FImageView for image data.
	
		Not actually a const function, may change the "ImageData" member.
	*/
	CORE_API const TArray< uint8 >& GetUncompressedImageData() const;

	/** Returns uncompressed image data as an FImageView, decompressing it on demand if needed.

		Note the FImageView does not have a copy of the data, it points at the Thumbnail UncompressedImageData.
		If the UncompressedImageData is freed or changed, the FImageView will be affected.
		
		Not actually a const function, may change the "ImageData" member.
	*/
	inline FImageView GetImage() const; // Note, you need to include ObjectThumbnail.inl to use this

	/** Copy Image into Thumbnail uncompressed image data.
		Existing compressed data, if any, is freed.
	
		Thumbnails are always BGRA8-sRGB.  The passed in Image can be other formats and conversion will be done if needed.
	*/
	inline void SetImage(const FImageView & Image); // Note, you need to include ObjectThumbnail.inl to use this
	
	/** Move Image data into Thumbnail (convert to BGRA8-SRGB if necessary) */
	inline void SetImage(FImage && Image); // Note, you need to include ObjectThumbnail.inl to use this

	/** Serializers */
	CORE_API void Serialize(FArchive& Ar);
	CORE_API void Serialize(FStructuredArchive::FSlot Slot);
	
	/** Compress image data. */
	CORE_API void CompressImageData();

	/** Decompress image data. */
	CORE_API void DecompressImageData();

	/** Empties the uncompressed image data. The next call to GetImage will have to decompress the image again. */
	CORE_API void EmptyUncompressedImageData();

	/**
	 * Calculates the memory usage of this FObjectThumbnail.
	 *
	 * @param	Ar	the FArchiveCountMem (or similar) archive that will store the results of the memory usage calculation.
	 */
	CORE_API void CountBytes( FArchive& Ar ) const;

	/**
	 * Calculates the amount of memory used by the compressed bytes array.
	 *
	 * @param	Ar	the FArchiveCountMem (or similar) archive that will store the results of the memory usage calculation.
	 */
	CORE_API void CountImageBytes_Compressed( FArchive& Ar ) const;

	/**
	 * Calculates the amount of memory used by the uncompressed bytes array.
	 *
	 * @param	Ar	the FArchiveCountMem (or similar) archive that will store the results of the memory usage calculation.
	 */
	CORE_API void CountImageBytes_Uncompressed( FArchive& Ar ) const;

	/** I/O operator */
	friend FArchive& operator<<( FArchive& Ar, FObjectThumbnail& Thumb )
	{
		if ( Ar.IsCountingMemory() )
		{
			Thumb.CountBytes(Ar);
		}
		else
		{
			Thumb.Serialize(Ar);
		}
		return Ar;
	}

	friend FArchive& operator<<( FArchive& Ar, const FObjectThumbnail& Thumb )
	{
		Thumb.CountBytes(Ar);
		return Ar;
	}

	/** Comparison operator */
	bool operator ==( const FObjectThumbnail& Other ) const
	{
		return	ImageWidth			== Other.ImageWidth
			&&	ImageHeight			== Other.ImageHeight
			&&	bIsDirty			== Other.bIsDirty
			&&	CompressedImageData	== Other.CompressedImageData;
	}

	bool operator !=( const FObjectThumbnail& Other ) const
	{
		return	ImageWidth			!= Other.ImageWidth
			||	ImageHeight			!= Other.ImageHeight
			||	bIsDirty			!= Other.bIsDirty
			||	CompressedImageData	!= Other.CompressedImageData;
	}

private:

	/** Thumbnail width (serialized) */
	int32 ImageWidth;

	/** Thumbnail height (serialized) */
	int32 ImageHeight;

	/** Compressed image data (serialized) */
	TArray< uint8 > CompressedImageData;

	/** Image data bytes */
	TArray< uint8 > ImageData;

	/** True if the thumbnail is dirty and should be regenerated at some point */
	bool bIsDirty;

	/** Whether the thumbnail has a backup on disk*/
	bool bLoadedFromDisk;

	/** Whether compressed data is JPEG (else PNG) */
	bool bIsJPEG;

	/** Whether this was saved AFTER custom-thumbnails for shared thumbnail asset types was supported */
	bool bCreatedAfterCustomThumbForSharedTypesEnabled;
};


/** Maps an object's full name to a thumbnail */
typedef TMap< FName, FObjectThumbnail > FThumbnailMap;


/** Wraps an object's full name and thumbnail */
struct FObjectFullNameAndThumbnail
{
	/** Full name of the object */
	FName ObjectFullName;
	
	/** Thumbnail data */
	const FObjectThumbnail* ObjectThumbnail;

	/** Offset in the file where the data is stored */
	int32 FileOffset;

	/** Constructor */
	FObjectFullNameAndThumbnail()
		: ObjectFullName(),
		  ObjectThumbnail( nullptr ),
		  FileOffset( 0 )
	{ }

	/** Constructor */
	FObjectFullNameAndThumbnail( const FName InFullName, const FObjectThumbnail* InThumbnail )
		: ObjectFullName( InFullName ),
		  ObjectThumbnail( InThumbnail ),
		  FileOffset( 0 )
	{ }

	/**
	 * Calculates the memory usage of this FObjectFullNameAndThumbnail.
	 *
	 * @param	Ar	the FArchiveCountMem (or similar) archive that will store the results of the memory usage calculation.
	 */
	CORE_API void CountBytes( FArchive& Ar ) const;

	/** I/O operator */
	friend FArchive& operator<<( FArchive& Ar, FObjectFullNameAndThumbnail& NameThumbPair )
	{
		if ( Ar.IsCountingMemory() )
		{
			NameThumbPair.CountBytes(Ar);
		}
		else
		{
			Ar << NameThumbPair.ObjectFullName << NameThumbPair.FileOffset;
		}
		return Ar;
	}

	friend FArchive& operator<<( FArchive& Ar, const FObjectFullNameAndThumbnail& NameThumbPair )
	{
		NameThumbPair.CountBytes(Ar);
		return Ar;
	}
};
