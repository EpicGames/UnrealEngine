// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(IMAGECORE_API)

#pragma once

#include "Misc/ObjectThumbnail.h"
#include "ImageCore.h"


FImageView FObjectThumbnail::GetImage() const
{
	const TArray< uint8 >& Data = FObjectThumbnail::GetUncompressedImageData();
	if ( Data.Num() == 0 )
	{
		return FImageView();
	}

	return FImageView( (void *)&Data[0],ImageWidth,ImageHeight,ERawImageFormat::BGRA8 );
}

void FObjectThumbnail::SetImage(const FImageView & Image)
{
	// Image must be converted to BGRA8 to store in Thumbnail
	// if Image is already BGRA8-SRGB then this is just a memcpy
	//	which is what we need anyway to copy the bytes into a new array
	//	so there is no inefficiency in just always using the FImage copy here
	FImage ImageBGRA8;
	Image.CopyTo(ImageBGRA8,ERawImageFormat::BGRA8,EGammaSpace::sRGB);
	SetImage( MoveTemp(ImageBGRA8) );
}
	
/** Move Image data into Thumbnail (convert to BGRA8-SRGB if necessary) */
void FObjectThumbnail::SetImage(FImage && Image)
{
	// change format if needed, nop if not
	// MoveFrom is discardable so it's okay if we just change it in place
	Image.ChangeFormat(ERawImageFormat::BGRA8,EGammaSpace::sRGB);

	ImageWidth  = Image.SizeX;
	ImageHeight = Image.SizeY;
	CompressedImageData.Reset();
	bIsJPEG = false;
	ImageData = MoveTemp(Image.RawData); // array move
	Image.Reset();
}

#endif
