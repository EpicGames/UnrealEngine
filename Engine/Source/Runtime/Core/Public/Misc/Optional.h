// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/ReturnFromOpenMode.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/IntrusiveUnsetOptionalState.h"
#include "Misc/OptionalFwd.h"
#include "Templates/MemoryOps.h"
#include "Templates/TypeHash.h" // Definitions of GetTypeHash for primitive types must be visible when GetTypeHash for TOptional is defined
#include "Templates/UnrealTemplate.h"
#include "Serialization/Archive.h"
#include <type_traits>

inline constexpr FNullOpt NullOpt{0};

namespace UE::Core::Private
{
	// Empty structure to replace bIsSet for intrusive optional types 
	struct FEmpty {};

	// Shared code for optionals to handle multiple implementations of IsSet depending on compiler support for constrained destructors
	// This class can be a friend of FIntrusiveUnsetOptionalState rather than all possible base classes.
	struct FOptional
	{
		template<typename Derived>
		[[nodiscard]] inline static constexpr bool IsSet(Derived* This)
		{
			if constexpr (Derived::bUsingIntrusiveUnsetState)
			{
				return !(This->TypedValue == FIntrusiveUnsetOptionalState{});
			}
			else
			{
				return This->bIsSet;
			}
		}
	};
}

/**
 * When we have an optional value IsSet() returns true, and GetValue() is meaningful.
 * Otherwise GetValue() is not meaningful.
 */
template<typename OptionalType>
struct TOptional
{
private:
	using FOptional = UE::Core::Private::FOptional;
	friend FOptional;
	static constexpr bool bUsingIntrusiveUnsetState = HasIntrusiveUnsetOptionalState<OptionalType>();

public:
	using ElementType = OptionalType;

#if UE_AUTORTFM
	// TOptional can be returned from AutoRTFM::Open() via a copy-construction in the closed,
	// so long as the inner type also supports copy-construction in the closed.
	static constexpr AutoRTFM::EReturnFromOpenMode AutoRTFMReturnFromOpenMode =
		(AutoRTFM::ReturnFromOpenModeFor<OptionalType> == AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed) ?
		AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed : AutoRTFM::EReturnFromOpenMode::Unsupported;
#endif

	/** Construct an OptionalType with a valid value. */
	[[nodiscard]] constexpr TOptional(const OptionalType& InValue)
		: TOptional(InPlace, InValue)
	{
	}
	[[nodiscard]] constexpr TOptional(OptionalType&& InValue)
		: TOptional(InPlace, MoveTempIfPossible(InValue))
	{
	}
	template <typename... ArgTypes>
		requires (std::is_constructible_v<OptionalType, ArgTypes...>)
	[[nodiscard]] explicit constexpr TOptional(EInPlace, ArgTypes&&... Args)
		: TypedValue(Forward<ArgTypes>(Args)...)
	{
		// If this fails to compile when trying to call TOptional(EInPlace, ...) with a non-public constructor,
		// do not make TOptional a friend.
		//
		// Instead, prefer this pattern:
		//
		//     class FMyType
		//     {
		//     private:
		//         struct FPrivateToken { explicit FPrivateToken() = default; };
		//
		//     public:
		//         // This has an equivalent access level to a private constructor,
		//         // as only friends of FMyType will have access to FPrivateToken,
		//         // but the TOptional constructor can legally call it since it's public.
		//         explicit FMyType(FPrivateToken, int32 Int, float Real, const TCHAR* String);
		//     };
		//
		//     // Won't compile if the caller doesn't have access to FMyType::FPrivateToken
		//     TOptional<FMyType> Opt(InPlace, FMyType::FPrivateToken{}, 5, 3.14f, TEXT("Banana"));
		//

		if constexpr (!bUsingIntrusiveUnsetState)
		{
			this->bIsSet = true;
		}
		else
		{
			// Ensure that a user doesn't emplace an unset state into the optional
			checkf(IsSet(), TEXT("TOptional::TOptional(EInPlace, ...) - optionals should not be unset by emplacement"));
		}
	}
	
	/** Construct an OptionalType with an invalid value. */
	[[nodiscard]] constexpr TOptional(FNullOpt)
		: TOptional()
	{
	}

	/** Construct an OptionalType intrusively with no value; i.e. unset */
	[[nodiscard]] constexpr TOptional() requires bUsingIntrusiveUnsetState
		: TypedValue(FIntrusiveUnsetOptionalState{})
	{
	}

	/** Construct an OptionalType with no value; i.e. unset */
	[[nodiscard]] constexpr TOptional() requires (!bUsingIntrusiveUnsetState)
		: UnsetValue()
	{
	}

	constexpr ~TOptional() requires std::is_trivially_destructible_v<OptionalType> = default;
	constexpr ~TOptional() requires (!std::is_trivially_destructible_v<OptionalType>)
	{
		// Destroy value but do not reconstruct an empty intrusive optional in its place
		if (IsSet())
		{
			DestroyValue();
		}
	}

	/** Copy/Move construction */
	[[nodiscard]] TOptional(const TOptional& Other)
		requires (std::is_copy_constructible_v<OptionalType> && !std::is_trivially_copy_constructible_v<OptionalType>)
		: TOptional()
	{
		bool bLocalIsSet = Other.IsSet();
		if constexpr (!bUsingIntrusiveUnsetState)
		{
			this->bIsSet = bLocalIsSet;
		}
		if (bLocalIsSet)
		{
			::new((void*)std::addressof(this->TypedValue)) OptionalType(Other.TypedValue);
		}
		// Default constructor initialized unset intrusive optional if necessary
	}
	[[nodiscard]] TOptional(const TOptional& Other) requires (std::is_copy_constructible_v<OptionalType> && std::is_trivially_copy_constructible_v<OptionalType>) = default;
	[[nodiscard]] TOptional(const TOptional& Other) requires (!std::is_copy_constructible_v<OptionalType>) = delete;

	[[nodiscard]] TOptional(TOptional&& Other)
		requires (std::is_move_constructible_v<OptionalType> && !std::is_trivially_move_constructible_v<OptionalType>)
		: TOptional()
	{
		bool bLocalIsSet = Other.IsSet();
		if constexpr (!bUsingIntrusiveUnsetState)
		{
			this->bIsSet = bLocalIsSet;
		}
		if (bLocalIsSet)
		{
			::new((void*)std::addressof(this->TypedValue)) OptionalType(MoveTempIfPossible(Other.TypedValue));
		}
		// Default constructor initialized unset intrusive optional if necessary
	}
	[[nodiscard]] TOptional(TOptional&& Other) requires (std::is_move_constructible_v<OptionalType> && std::is_trivially_move_constructible_v<OptionalType>) = default;
	[[nodiscard]] TOptional(TOptional&& Other) requires (!std::is_move_constructible_v<OptionalType>) = delete;

	TOptional& operator=(const TOptional& Other)
		// Use is_constructible traits instead of is_assignable, because it uses Emplace, which always uses construction.
		// If trivially copy constructible but not trivially copy assignable, handle this here too, so we don't accidentally copy assign.
		requires (std::is_copy_constructible_v<OptionalType> && (!std::is_trivially_copy_constructible_v<OptionalType> || !std::is_trivially_copy_assignable_v<OptionalType>))
	{
		if (&Other != this)
		{
			if (Other.IsSet())
			{
				Emplace(Other.GetValue());
			}
			else
			{
				Reset();
			}
		}
		return *this;
	}
	TOptional& operator=(const TOptional& Other) requires (std::is_copy_assignable_v<OptionalType> && std::is_trivially_copy_constructible_v<OptionalType> && std::is_trivially_copy_assignable_v<OptionalType>) = default;
	TOptional& operator=(const TOptional& Other) requires (!std::is_copy_constructible_v<OptionalType>) = delete;

	TOptional& operator=(TOptional&& Other)
		// Use is_constructible traits instead of is_assignable, because it uses Emplace, which always uses construction.
		// If trivially move constructible but not trivially move assignable, handle this here too, so we don't accidentally move assign.
		requires (std::is_move_constructible_v<OptionalType> && (!std::is_trivially_move_constructible_v<OptionalType> || !std::is_trivially_move_assignable_v<OptionalType>))
	{
		if (&Other != this)
		{
			if(Other.IsSet())
			{
				Emplace(MoveTempIfPossible(Other.GetValue()));
			}
			else
			{
				Reset();
			}
		}
		return *this;
	}
	TOptional& operator=(TOptional&& Other) requires (std::is_move_assignable_v<OptionalType> && std::is_trivially_move_constructible_v<OptionalType> && std::is_trivially_move_assignable_v<OptionalType>) = default;
	TOptional& operator=(TOptional&& Other) requires (!std::is_move_constructible_v<OptionalType>) = delete;

	TOptional& operator=(const OptionalType& InValue)
	{
		if (std::addressof(InValue) != std::addressof(this->TypedValue))
		{
			Emplace(InValue);
		}
		return *this;
	}
	TOptional& operator=(OptionalType&& InValue)
	{
		if (std::addressof(InValue) != std::addressof(this->TypedValue))
		{
			Emplace(MoveTempIfPossible(InValue));
		}
		return *this;
	}

	void Reset()
	{
		if (IsSet())
		{
			DestroyValue();
			if constexpr (bUsingIntrusiveUnsetState)
			{
				::new((void*)std::addressof(this->TypedValue)) OptionalType(FIntrusiveUnsetOptionalState{});
			}
			else
			{
				this->UnsetValue = {};
				this->bIsSet = false;
			}
		}
	}

	template <typename... ArgsType>
		requires (std::is_constructible_v<OptionalType, ArgsType...>)
	OptionalType& Emplace(ArgsType&&... Args)
	{
		// Destroy the member in-place before replacing it - a bit nasty, but it'll work since we don't support exceptions
		if constexpr (bUsingIntrusiveUnsetState)
		{
			DestroyValue();
		}
		else
		{
			if (IsSet())
			{
				DestroyValue();
			}
		}

		// If this fails to compile when trying to call Emplace with a non-public constructor,
		// do not make TOptional a friend.
		//
		// Instead, prefer this pattern:
		//
		//     class FMyType
		//     {
		//     private:
		//         struct FPrivateToken { explicit FPrivateToken() = default; };
		//
		//     public:
		//         // This has an equivalent access level to a private constructor,
		//         // as only friends of FMyType will have access to FPrivateToken,
		//         // but Emplace can legally call it since it's public.
		//         explicit FMyType(FPrivateToken, int32 Int, float Real, const TCHAR* String);
		//     };
		//
		//     TOptional<FMyType> Opt:
		//
		//     // Won't compile if the caller doesn't have access to FMyType::FPrivateToken
		//     Opt.Emplace(FMyType::FPrivateToken{}, 5, 3.14f, TEXT("Banana"));
		//
		OptionalType* Result = ::new((void*)std::addressof(this->TypedValue)) OptionalType(Forward<ArgsType>(Args)...);

		if constexpr (!bUsingIntrusiveUnsetState)
		{
			this->bIsSet = true;
		}
		else
		{
			// Ensure that a user doesn't emplace an unset state into the optional
			checkf(IsSet(), TEXT("TOptional::Emplace(...) - optionals should not be unset by an emplacement"));
		}

		return *Result;
	}

	[[nodiscard]] friend constexpr bool operator==(const TOptional& Lhs, const TOptional& Rhs)
	{
		bool bIsLhsSet = Lhs.IsSet();
		bool bIsRhsSet = Rhs.IsSet();

		if (bIsLhsSet != bIsRhsSet)
		{
			return false;
		}
		if (!bIsLhsSet) // both unset
		{
			return true;
		}

		return Lhs.TypedValue == Rhs.TypedValue;
	}

	[[nodiscard]] friend constexpr bool operator!=(const TOptional& Lhs, const TOptional& Rhs)
	{
		return !(Lhs == Rhs);
	}

	void Serialize(FArchive& Ar)
	{
		bool bOptionalIsSet = IsSet();
		if (Ar.IsLoading())
		{
			bool bOptionalWasSaved = false;
			Ar << bOptionalWasSaved;
			if (bOptionalWasSaved)
			{
				if (!bOptionalIsSet)
				{
					Emplace();
				}
				Ar << GetValue();
			}
			else
			{
				Reset();
			}
		}
		else
		{
			Ar << bOptionalIsSet;
			if (bOptionalIsSet)
			{
				Ar << GetValue();
			}
		}
	}

	/** @return true when the value is meaningful; false if calling GetValue() is undefined. */
	[[nodiscard]] constexpr bool IsSet() const
	{
		return FOptional::IsSet(this);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT explicit constexpr operator bool() const
	{
		return IsSet();
	}

	/** @return The optional value; undefined when IsSet() returns false. */
	[[nodiscard]] constexpr OptionalType& GetValue() &
	{
		checkf(IsSet(), TEXT("It is an error to call GetValue() on an unset TOptional. Please either check IsSet() or use Get(DefaultValue) instead."));
		return this->TypedValue;
	}
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr const OptionalType& GetValue() const&
	{
		return const_cast<TOptional*>(this)->GetValue();
	}
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr OptionalType&& GetValue() &&
	{
		return MoveTempIfPossible(GetValue());
	}

	[[nodiscard]] constexpr OptionalType* operator->()
	{
		return std::addressof(GetValue());
	}
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr const OptionalType* operator->() const
	{
		return const_cast<TOptional*>(this)->operator->();
	}

	[[nodiscard]] constexpr OptionalType& operator*() &
	{
		return GetValue();
	}
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr const OptionalType& operator*() const&
	{
		return const_cast<TOptional*>(this)->operator*();
	}
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr OptionalType&& operator*() &&
	{
		return MoveTempIfPossible(operator*());
	}

	/** @return The optional value when set; DefaultValue otherwise. */
	[[nodiscard]] constexpr const OptionalType& Get(const OptionalType& DefaultValue UE_LIFETIMEBOUND) const UE_LIFETIMEBOUND
	{
		return IsSet() ? this->TypedValue : DefaultValue;
	}

	/** @return A pointer to the optional value when set, nullptr otherwise. */
	[[nodiscard]] constexpr OptionalType* GetPtrOrNull()
	{
		return IsSet() ? std::addressof(this->TypedValue) : nullptr;
	}
	[[nodiscard]] UE_FORCEINLINE_HINT constexpr const OptionalType* GetPtrOrNull() const
	{
		return const_cast<TOptional*>(this)->GetPtrOrNull();
	}

private:
	/** 
	 * Destroys the value, must only be called if the value is set, and callers must then mark the value unset or construct a new value in place. 
	 */
	UE_FORCEINLINE_HINT constexpr void DestroyValue()
	{
		DestructItem(std::addressof(this->TypedValue));
	}

	// This dummy type exists to prevent the OptionalType from being constructed when
	// the union needs to be constructed, when in constexpr contexts, the union needs
	// to be constructed to something.
	struct FUnsetType
	{
		[[nodiscard]] UE_REWRITE constexpr FUnsetType()
		{
		}
	};

	// If we can have an explicitly conditional trivial destructor, we can declare our members inline and avoid the base class
	union
	{
		FUnsetType UnsetValue;
		OptionalType TypedValue;
	};
	// Set flag takes up no space if bIsUsingIntrusiveUnsetState, otherwise is zero-initialized to false
	UE_NO_UNIQUE_ADDRESS std::conditional_t<bUsingIntrusiveUnsetState, UE::Core::Private::FEmpty, bool> bIsSet = {};
};

template<typename OptionalType>
FArchive& operator<<(FArchive& Ar, TOptional<OptionalType>& Optional)
{
	Optional.Serialize(Ar);
	return Ar;
}

template<typename OptionalType>
[[nodiscard]] inline auto GetTypeHash(const TOptional<OptionalType>& Optional) -> decltype(GetTypeHash(*Optional))
{
	return Optional.IsSet() ? GetTypeHash(*Optional) : 0;
}

/**
 * Trait which determines whether or not a type is a TOptional.
 */
template <typename T> static constexpr bool TIsTOptional_V                              = false;
template <typename T> static constexpr bool TIsTOptional_V<               TOptional<T>> = true;
template <typename T> static constexpr bool TIsTOptional_V<const          TOptional<T>> = true;
template <typename T> static constexpr bool TIsTOptional_V<      volatile TOptional<T>> = true;
template <typename T> static constexpr bool TIsTOptional_V<const volatile TOptional<T>> = true;
