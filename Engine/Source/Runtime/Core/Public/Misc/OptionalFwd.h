// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_exports

/**
 * When we have an optional value IsSet() returns true, and GetValue() is meaningful.
 * Otherwise GetValue() is not meaningful.
 */
template<typename OptionalType>
struct TOptional;

struct FNullOpt
{
	explicit constexpr FNullOpt(int) {}
};

// IWYU pragma: end_exports
