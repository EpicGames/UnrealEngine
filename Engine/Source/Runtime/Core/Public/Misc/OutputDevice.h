// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "CoreTypes.h"
#include "Logging/LogVerbosity.h"
#include "Misc/VarArgs.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Templates/IsValidVariadicFunctionArg.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "UObject/CoreReflectedTypeAccessors.h"

class FString;
class FText;

namespace UE { class FLogRecord; }

#ifndef USE_DEBUG_LOGGING
#define USE_DEBUG_LOGGING 1
#endif

#ifndef USE_EVENT_LOGGING
#define USE_EVENT_LOGGING 1
#endif

#if !PLATFORM_SUPPORTS_COLORIZED_OUTPUT_DEVICE
	// don't support colorized text on consoles
	#define SET_WARN_COLOR(Color)
	#define SET_WARN_COLOR_AND_BACKGROUND(Color, Bkgrnd)
	#define CLEAR_WARN_COLOR() 
#else

/*-----------------------------------------------------------------------------
Colorized text.

To use colored text from a commandlet, you use the SET_WARN_COLOR macro with
one of the following standard colors. Then use CLEAR_WARN_COLOR to return
to default.

To use the standard colors, you simply do this:
SET_WARN_COLOR(COLOR_YELLOW);

You can specify a background color by appending it to the foreground:
SET_WARN_COLOR, COLOR_YELLOW COLOR_DARK_RED);

This will have bright yellow text on a dark red background.

Or you can make your own in the format:

ForegroundRed | ForegroundGreen | ForegroundBlue | ForegroundBright | BackgroundRed | BackgroundGreen | BackgroundBlue | BackgroundBright
where each value is either 0 or 1 (can leave off trailing 0's), so
blue on bright yellow is "00101101" and red on black is "1"

An empty string reverts to the normal gray on black.
-----------------------------------------------------------------------------*/

// putting them in a namespace to protect against future name conflicts
namespace OutputDeviceColor
{
	inline const TCHAR* const COLOR_BLACK = TEXT("0000");

	inline const TCHAR* const COLOR_DARK_RED = TEXT("1000");
	inline const TCHAR* const COLOR_DARK_GREEN = TEXT("0100");
	inline const TCHAR* const COLOR_DARK_BLUE = TEXT("0010");
	inline const TCHAR* const COLOR_DARK_YELLOW = TEXT("1100");
	inline const TCHAR* const COLOR_DARK_CYAN = TEXT("0110");
	inline const TCHAR* const COLOR_DARK_PURPLE = TEXT("1010");
	inline const TCHAR* const COLOR_DARK_WHITE = TEXT("1110");
	inline const TCHAR* const COLOR_GRAY = COLOR_DARK_WHITE;

	inline const TCHAR* const COLOR_RED = TEXT("1001");
	inline const TCHAR* const COLOR_GREEN = TEXT("0101");
	inline const TCHAR* const COLOR_BLUE = TEXT("0011");
	inline const TCHAR* const COLOR_YELLOW = TEXT("1101");
	inline const TCHAR* const COLOR_CYAN = TEXT("0111");
	inline const TCHAR* const COLOR_PURPLE = TEXT("1011");
	inline const TCHAR* const COLOR_WHITE = TEXT("1111");

	inline const TCHAR* const COLOR_NONE = TEXT("");
}

using namespace OutputDeviceColor;


// let a console or (UE_BUILD_SHIPPING || UE_BUILD_TEST) define it to nothing
#ifndef SET_WARN_COLOR

/**
* Set the console color with Color or a Color and Background color
*/
#define SET_WARN_COLOR(Color) \
	UE_LOGF(LogHAL, SetColor, "%ls", Color);
#define SET_WARN_COLOR_AND_BACKGROUND(Color, Bkgrnd) \
	UE_LOGF(LogHAL, SetColor, "%ls%ls", Color, Bkgrnd);

/**
* Return color to it's default
*/
#define CLEAR_WARN_COLOR() \
	UE_LOGF(LogHAL, SetColor, "%ls", COLOR_NONE);

#endif
#endif

/**
 * Enum that defines how the log times are to be displayed.
 */
namespace ELogTimes
{
	enum Type : int
	{
		// Do not display log timestamps
		None,

		// Display log timestamps in UTC
		UTC,

		// Display log timestamps in seconds elapsed since GStartTime
		SinceGStartTime,

		// Display log timestamps in local time
		Local,

		// Display log timestamps in timecode format
		Timecode
	};
}

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<ELogTimes::Type>();
#endif

class FName;

// An output device.
class FOutputDevice
{
public:
	CORE_API FOutputDevice();

	CORE_API FOutputDevice(FOutputDevice&&);
	CORE_API FOutputDevice(const FOutputDevice&);
	CORE_API FOutputDevice& operator=(FOutputDevice&&);
	CORE_API FOutputDevice& operator=(const FOutputDevice&);

	CORE_API virtual ~FOutputDevice();

	// FOutputDevice interface.
	virtual void Serialize( const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category ) = 0;
	virtual void Serialize( const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, const double Time )
	{
		Serialize( V, Verbosity, Category );
	}

	CORE_API virtual void SerializeRecord(const UE::FLogRecord& Record);

	virtual void Flush()
	{
	}

	/**
	 * Closes output device and cleans up. This can't happen in the destructor
	 * as we might have to call "delete" which cannot be done for static/ global
	 * objects.
	 */
	virtual void TearDown()
	{
	}

	void SetSuppressEventTag(bool bInSuppressEventTag)
	{
		bSuppressEventTag = bInSuppressEventTag;
	}
	UE_FORCEINLINE_HINT bool GetSuppressEventTag() const	{	return bSuppressEventTag;	}
	void SetAutoEmitLineTerminator(bool bInAutoEmitLineTerminator)
	{
		bAutoEmitLineTerminator = bInAutoEmitLineTerminator;
	}
	UE_FORCEINLINE_HINT bool GetAutoEmitLineTerminator() const	{	return bAutoEmitLineTerminator;	}

	/** 
	 * Dumps the contents of this output device's buffer to an archive (supported by output device that have a memory buffer) 
	 * @param Ar Archive to dump the buffer to
	 */
	virtual void Dump(class FArchive& Ar)
	{
	}

	/**
	* @return whether this output device is a memory-only device
	*/
	virtual bool IsMemoryOnly() const
	{
		return false;
	}

	/**
	 * @return whether this output device can be used on any thread.
	 */
	virtual bool CanBeUsedOnAnyThread() const
	{
		return false;
	}

	/**
	 * @return whether this output device can be used from multiple threads simultaneously without any locking
	 */
	virtual bool CanBeUsedOnMultipleThreads() const
	{
		return false;
	}

	/**
	 * @return whether this output device can be used after a panic (crash or fatal error) has been flagged.
	 * @note The return value is cached by AddOutputDevice because calling this during a panic may fail.
	 */
	virtual bool CanBeUsedOnPanicThread() const
	{
		return false;
	}

	// Simple text printing.
	CORE_API void Log( const TCHAR* S );
	CORE_API void Log( ELogVerbosity::Type Verbosity, const TCHAR* S );
	CORE_API void Log( const FName& Category, ELogVerbosity::Type Verbosity, const TCHAR* Str );
	CORE_API void Log( const FString& S );
	CORE_API void Log( const FText& S );
	CORE_API void Log( ELogVerbosity::Type Verbosity, const FString& S );
	CORE_API void Log( const FName& Category, ELogVerbosity::Type Verbosity, const FString& S );

private:
	CORE_API void VARARGS LogfImpl(const TCHAR* Fmt, ...);
	CORE_API void VARARGS LogfImpl(ELogVerbosity::Type Verbosity, const TCHAR* Fmt, ...);
	CORE_API void VARARGS CategorizedLogfImpl(const FName& Category, ELogVerbosity::Type Verbosity, const TCHAR* Fmt, ...);

public:
	template <typename FmtType>
	void Logf(const FmtType& Fmt)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		return Log((const TCHAR*)Fmt);
	}

	template <typename FmtType, typename... Types>
	inline void Logf(const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FOutputDevice::Logf");

		LogfImpl((const TCHAR*)Fmt, Args...);
	}

	template <typename FmtType, typename... Types>
	inline void Logf(ELogVerbosity::Type Verbosity, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FOutputDevice::Logf");

		LogfImpl(Verbosity, (const TCHAR*)Fmt, Args...);
	}

	template <typename FmtType, typename... Types>
	inline void CategorizedLogf(const FName& Category, ELogVerbosity::Type Verbosity, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FOutputDevice::CategorizedLogf");

		CategorizedLogfImpl(Category, Verbosity, (const TCHAR*)Fmt, Args...);
	}

protected:
	/** Whether to output the 'Log: ' type front... */
	bool bSuppressEventTag;
	/** Whether to output a line-terminator after each log call... */
	bool bAutoEmitLineTerminator;
};



