// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogVerbosity.h"
#include "Misc/OutputDeviceError.h"
#include "UObject/NameTypes.h"

class FOutputDeviceAnsiError : public FOutputDeviceError
{
public:
	/** Constructor, initializing member variables */
	CORE_API FOutputDeviceAnsiError();

	/**
	* Serializes the passed in data unless the current event is suppressed.
	*
	* @param	Data	Text to log
	* @param	Event	Event name used for suppression purposes
	*/
	CORE_API virtual void Serialize(const TCHAR* Msg, ELogVerbosity::Type Verbosity, const class FName& Category) override;

	virtual bool CanBeUsedOnAnyThread() const override
	{
		return true;
	}

	/**
	* Error handling function that is being called from within the system wide global
	* error handler, e.g. using structured exception handling on the PC.
	*/
	CORE_API void HandleError() override;

private:

	int32		ErrorPos;
};

