// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogVerbosity.h"
#include "Misc/AssertionMacros.h"
#include "Misc/OutputDevice.h"

class FArchive;

/**
* Output device wrapping any kind of FArchive.  Note: Works in any build configuration.
*/
class FOutputDeviceArchiveWrapper : public FOutputDevice
{
public:
	/**
	* Constructor, initializing member variables.
	*
	* @param InArchive	Archive to use, must not be nullptr.  Does not take ownership of the archive, clean up or delete the archive independently!
	*/
	FOutputDeviceArchiveWrapper(FArchive* InArchive)
		: LogAr(InArchive)
	{
		check(InArchive);
	}

	// FOutputDevice interface
	CORE_API virtual void Flush() override;
	CORE_API virtual void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category) override;
	// End of FOutputDevice interface

private:
	FArchive* LogAr;
};
