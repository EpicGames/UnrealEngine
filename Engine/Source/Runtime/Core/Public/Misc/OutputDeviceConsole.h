// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/OutputDevice.h"
#include "Containers/UnrealString.h"

/**
 * This class servers as the base class for console window output.
 */
class FOutputDeviceConsole
	: public FOutputDevice
{
public:

	/**
	 * Shows or hides the console window. 
	 *
	 * @param ShowWindow	Whether to show (true) or hide (false) the console window.
	 */
	virtual void Show( bool ShowWindow )=0;

	/** 
	 * Returns whether console is currently shown or not.
	 *
	 * @return true if console is shown, false otherwise.
	 */
	virtual bool IsShown()=0;

	/** 
	 * Returns whether the application is already attached to a console window.
	 *
	 * @return true if console is attached, false otherwise.
	 */
	virtual bool IsAttached()
	{
		return false;
	}

	/**
	 * Sets the INI file name to write console settings to.
	 *
	 * @param InFilename The INI file name to set.
	 */
	void SetIniFilename(const TCHAR* InFilename) 
	{
		IniFilename = InFilename;
	}

protected:

	/** Ini file name to write console settings to. */
	FString IniFilename;
};
