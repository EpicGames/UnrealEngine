// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogVerbosity.h"
#include "Misc/OutputDevice.h"
#include "UObject/NameTypes.h"

#include <atomic>

class FOutputDeviceDebug : public FOutputDevice
{
public:
	/**
	* Serializes the passed in data unless the current event is suppressed.
	*
	* @param	Data	Text to log
	* @param	Event	Event name used for suppression purposes
	*/
	CORE_API virtual void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const FName& Category, double Time) override;

	CORE_API virtual void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const FName& Category) override;

	CORE_API virtual void SerializeRecord(const UE::FLogRecord& Record) override;

	virtual bool CanBeUsedOnAnyThread() const override
	{
		return true;
	}

	CORE_API virtual bool CanBeUsedOnMultipleThreads() const override;

	virtual bool CanBeUsedOnPanicThread() const override
	{
		return true;
	}

private:
	CORE_API void ConditionalTickAsync(double Time);
	CORE_API void TickAsync();

	std::atomic<double> LastTickTime = 0.0;
	std::atomic<bool> bSerializeAsJson = false;
};
