// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/OutputDevice.h"

// Error device.
class FOutputDeviceError : public FOutputDevice
{
public:
	virtual bool CanBeUsedOnPanicThread() const override
	{
		return true;
	}

	virtual void HandleError()=0;

	/** Sets the location of the instruction that raise the next error */
	void SetErrorProgramCounter(void* InProgramCounter)
	{
		ProgramCounter = InProgramCounter;
	}

	/* Returns the instruction location of where an error occurred */
	void* GetErrorProgramCounter() const
	{
		return ProgramCounter;
	}

private:
	void* ProgramCounter = nullptr;
};

