// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/Runnable.h"
#include "HAL/ThreadSafeCounter.h"
#include "Logging/LogVerbosity.h"
#include "Misc/OutputDevice.h"
#include "Misc/ScopeLock.h"
#include "Misc/SingleThreadRunnable.h"
#include "Serialization/Archive.h"
#include "Templates/Atomic.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

class FRunnableThread;


/** string added to the filename of timestamped backup log files */
#define BACKUP_LOG_FILENAME_POSTFIX TEXT("-backup-")

/**
* Provides a thread-safe serialization interface with a background thread doing the actual writes.
* [] tags identify which thread owns a variable or function
*/
class FAsyncWriter : public FRunnable, public FSingleThreadRunnable, public FArchive
{
	/** Thread to run the worker FRunnable on. Serializes the ring buffer to disk. */
	volatile FRunnableThread* Thread;
	/** Stops this thread */
	FThreadSafeCounter StopTaskCounter;

	/** Writer archive */
	FArchive& Ar;
	/** Data ring buffer */
	TArray<uint8> Buffer;
	/** [WRITER THREAD] Position where the unserialized data starts in the buffer */
	TAtomic<int32> BufferStartPos;
	/** [CLIENT THREAD] Position where the unserialized data ends in the buffer (such as if (BufferEndPos > BufferStartPos) Length = BufferEndPos - BufferStartPos; */
	TAtomic<int32> BufferEndPos;
	/** [CLIENT THREAD] Sync object for the buffer pos */
	FCriticalSection BufferPosCritical;
	/** [CLIENT/WRITER THREAD] Outstanding serialize request counter. This is to make sure we flush all requests. */
	FThreadSafeCounter SerializeRequestCounter;
	/** [CLIENT/WRITER THREAD] Tells the writer thread, the client requested flush. */
	FThreadSafeCounter WantsArchiveFlush;
	/** Sync object for buffer flushes in forkable mode, when forking hasn't occurred yet. */
	FCriticalSection RunCritical;

	/** [WRITER THREAD] Last time the archive was flushed. used in threaded situations to flush the underlying archive at a certain maximum rate. */
	double LastArchiveFlushTime;

	/** [WRITER THREAD] Flushes the archive and reset the flush timer. */
	CORE_API void FlushArchiveAndResetTimer();

	/** [WRITER THREAD] Serialize the contents of the ring buffer to disk */
	CORE_API void SerializeBufferToArchive();

	/** [CLIENT THREAD] Flush the memory buffer (doesn't force the archive to flush). Can only be used from inside of BufferPosCritical lock. */
	CORE_API void FlushBuffer();

public:

	enum class EThreadNameOption : uint8
	{
		FileName,   // Appends the filename in the threadname: FAsyncWriter_FileName
		Sequential, // Uses a global sequential number to append to the threadname: FAsyncWriter_1
	};

	CORE_API FAsyncWriter(FArchive& InAr, FAsyncWriter::EThreadNameOption NameOption=FAsyncWriter::EThreadNameOption::FileName);

	CORE_API virtual ~FAsyncWriter();

	/** [CLIENT THREAD] Serialize data to buffer that will later be saved to disk by the async thread */
	CORE_API virtual void Serialize(void* InData, int64 Length) override;

	/** Flush all buffers to disk */
	CORE_API void Flush();

	//~ Begin FRunnable Interface.
	CORE_API virtual bool Init();
	CORE_API virtual uint32 Run();
	CORE_API virtual void Stop();
	virtual FSingleThreadRunnable* GetSingleThreadInterface() override { return this; }
	//~ End FRunnable Interface

protected:
	//~ Begin FSingleThreadRunnable Interface.
	/** [CLIENT THREAD] A substitute for Run() for when threading is disabled. */
	CORE_API virtual void Tick() override;
	//~ End FSingleThreadRunnable Interface

};

enum class EByteOrderMark : int8
{
	UTF8,
	Unspecified,
};

/**
* File output device (Note: Only works if ALLOW_LOG_FILE && !NO_LOGGING is true, otherwise Serialize does nothing).
*/
class FOutputDeviceFile : public FOutputDevice
{
public:
	/**
	* Constructor, initializing member variables.
	*
	* @param InFilename	Filename to use, can be nullptr. If null, a file name will be automatically generated. If a filename is specified but cannot be opened
	*                   because it is already open/used by another process, the implementation will try to generate a new name automatically, until the a file
	*                   is created or the number of trials exhausted (32). 
	* @param bDisableBackup If true, existing files will not be backed up
	* @param bCreateWriterLazily If true, delay the creation of the file until something needs to be written, otherwise, open it immediatedly.
	* @param FileOpenedCallback If bound, invoked when the output file is successfully opened, passing the actual filename.
	*/
	CORE_API FOutputDeviceFile(const TCHAR* InFilename = nullptr, bool bDisableBackup = false, bool bAppendIfExists = false, bool bCreateWriterLazily = true, TFunction<void(const TCHAR*)> FileOpenedCallback = TFunction<void(const TCHAR*)>());

	/**
	* Destructor to perform teardown
	*
	*/
	CORE_API ~FOutputDeviceFile();

	/** Sets the filename that the output device writes to.  If the output device was already writing to a file, closes that file. */
	CORE_API void SetFilename(const TCHAR* InFilename);

	//~ Begin FOutputDevice Interface.
	/**
	* Closes output device and cleans up. This can't happen in the destructor
	* as we have to call "delete" which cannot be done for static/ global
	* objects.
	*/
	CORE_API void TearDown() override;

	/**
	* Flush the write cache so the file isn't truncated in case we crash right
	* after calling this function.
	*/
	CORE_API void Flush() override;

	CORE_API virtual void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category, const double Time) override;
	CORE_API virtual void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category) override;
	virtual bool CanBeUsedOnAnyThread() const override
	{
		return true;
	}
	virtual bool CanBeUsedOnPanicThread() const override
	{
		return true;
	}
	//~ End FOutputDevice Interface.

	/** Creates a backup copy of a log file if it already exists */
	static CORE_API void CreateBackupCopy(const TCHAR* Filename);

	/** Checks if the filename represents a backup copy of a log file */
	static CORE_API bool IsBackupCopy(const TCHAR* Filename);

	/** Add a category name to our inclusion filter. As soon as one inclusion exists, all others will be ignored */
	CORE_API void IncludeCategory(const class FName& InCategoryName);

	/** Returns the filename associated with this output device */
	const TCHAR* GetFilename() const { return Filename; }

	CORE_API bool IsOpened() const;

private:

	/** Writes to a file on a separate thread */
	FAsyncWriter* AsyncWriter;
	/** Archive used by the async writer */
	FArchive* WriterArchive;
	/** In bound, invoked when the log file is open successfully for writing, reporting the actual log filename. */
	TFunction<void(const TCHAR*)> OnFileOpenedFn;

	TCHAR Filename[1024];
	bool AppendIfExists;
	bool Dead;
	bool bCreatingWriter;

	/** If true, existing files will not be backed up */
	bool bDisableBackup;

	/** Internal data for category inclusion. Must be declared inside CPP file as it uses a TSet<FName> */
	struct FCategoryInclusionInternal;

	TUniquePtr<FCategoryInclusionInternal> CategoryInclusionInternal;

	CORE_API void WriteRaw(const TCHAR* C);

	/** Creates the async writer and its archive. Returns true if successful.  */
	CORE_API bool CreateWriter(uint32 MaxAttempts = 32);

	CORE_API void WriteByteOrderMarkToArchive(EByteOrderMark ByteOrderMark);
};
