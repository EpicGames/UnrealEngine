// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringFwd.h"
#include "Logging/LogVerbosity.h"
#include "Misc/OutputDevice.h"

class FArchive;
class FName;
class FString;

/** Helper functions used by FOutputDevice derived classes **/
struct FOutputDeviceHelper
{
	/** Append a formatted log line to the string builder. */
	static CORE_API void AppendFormatLogLine(FWideStringBuilderBase& Output, ELogVerbosity::Type Verbosity, const FName& Category, const TCHAR* Message = nullptr, ELogTimes::Type LogTime = ELogTimes::None, double Time = -1.0, int32* OutCategoryIndex = nullptr);
	static CORE_API void AppendFormatLogLine(FUtf8StringBuilderBase& Output, ELogVerbosity::Type Verbosity, const FName& Category, const TCHAR* Message = nullptr, ELogTimes::Type LogTime = ELogTimes::None, double Time = -1.0, int32* OutCategoryIndex = nullptr);

	/**
	 * Formats a log line with date, time, category and verbosity prefix
	 * @param Verbosity Message verbosity
	 * @param Category Message category
	 * @param Message Optional message text. If nullptr, only the date/time/category/verbosity prefix will be returned
	 * @param LogTime Time format
	 * @param Time Time in seconds
	 * @param OutCategoryIndex (if non-null) The index of the category within the return string is written here, or INDEX_NONE if the category is suppressed
	 * @returns Formatted log line
	 */
	static CORE_API FString FormatLogLine(ELogVerbosity::Type Verbosity, const FName& Category, const TCHAR* Message = nullptr, ELogTimes::Type LogTime = ELogTimes::None, double Time = -1.0, int32* OutCategoryIndex = nullptr);

	/**
	 * Formats, casts to ANSI char and serializes a message to archive. Optimized for small number of allocations and Serialize calls
	 * @param Output Output archive 
	 * @param Message Log message
	 * @param Verbosity Message verbosity
	 * @param Category Message category
	 * @param Time Message time
	 * @param bSuppressEventTag True if the message date/time prefix should be suppressed
	 * @param bAutoEmitLineTerminator True if the message should be automatically appended with a line terminator
	 **/
	static CORE_API void FormatCastAndSerializeLine(FArchive& Output, const TCHAR* Message, ELogVerbosity::Type Verbosity, const FName& Category, double Time, bool bSuppressEventTag, bool bAutoEmitLineTerminator);
};
