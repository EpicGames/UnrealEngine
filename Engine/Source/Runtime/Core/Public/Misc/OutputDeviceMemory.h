// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "Logging/LogVerbosity.h"
#include "Misc/OutputDevice.h"
#include "Serialization/Archive.h"
#include "UObject/NameTypes.h"

/**
 * Memory output device. Logs only into pre-allocated memory buffer.
 */
class FOutputDeviceMemory : public FOutputDevice
{
public:
	/** 
	 * Constructor, initializing member variables.
	 *
	 * @param InPreserveSize	Bytes of the rung buffer not to overwrite (startup info etc)
	 * @param InBufferSize		Maximum size of the memory ring buffer
	 */
	CORE_API FOutputDeviceMemory(int32 InPreserveSize = 256 * 1024, int32 InBufferSize = 2048 * 1024, bool bInSuppressEventTag = false);
	
	CORE_API ~FOutputDeviceMemory();

	/** Dumps the contents of the buffer to an archive */
	CORE_API virtual void Dump(FArchive& Ar) override;
	/** Deserialize the contents to a buffer */
	CORE_API bool DeserializeToBuffer(TArrayView<ANSICHAR> OutBuffer, int32 InOffsetInBytes = 0) const;

	//~ Begin FOutputDevice Interface.
	/**
	 * Closes output device and cleans up. This can't happen in the destructor
	 * as we have to call "delete" which cannot be done for static/ global
	 * objects.
	 */
	CORE_API virtual void TearDown() override;

	/**
	 * Flush the write cache so the file isn't truncated in case we crash right
	 * after calling this function.
	 */
	CORE_API virtual void Flush() override;

	CORE_API virtual void Serialize( const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category, const double Time ) override;
	CORE_API virtual void Serialize( const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category) override;
	virtual bool CanBeUsedOnAnyThread() const override
	{
		return true;
	}
	virtual bool CanBeUsedOnPanicThread() const override
	{
		return true;
	}
	virtual bool IsMemoryOnly() const override
	{
		return true;
	}
	//~ End FOutputDevice Interface.

private:
	
	/** Serialize cast data to the actual memory buffer */
	CORE_API void SerializeToBuffer(ANSICHAR* Data, int32 Length);


	class FOutputDeviceMemoryProxyArchive : public FArchive
	{
		FOutputDeviceMemory& OutputDevice;
	public:
		FOutputDeviceMemoryProxyArchive(FOutputDeviceMemory& InOutputDevice)
			: OutputDevice(InOutputDevice)
		{}
		virtual void Serialize(void* V, int64 Length) override
		{
			OutputDevice.SerializeToBuffer((ANSICHAR*)V, Length / sizeof(ANSICHAR));
		}
	} ArchiveProxy;

	/** Ring buffer */
	TArray<ANSICHAR> Buffer;
	/** Position where data starts in the buffer */
	int32 BufferStartPos;
	/** Used data size */
	int32 BufferLength;
	/** Amount of data not to overwrite */
	int32 PreserveSize;
	/** Sync object for the buffer pos */
	FCriticalSection BufferPosCritical;
};
