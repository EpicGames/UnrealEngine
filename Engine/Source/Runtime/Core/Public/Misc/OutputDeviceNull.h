// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogVerbosity.h"
#include "Misc/OutputDevice.h"

// Null output device.
class FOutputDeviceNull : public FOutputDevice
{
public:
	/**
	* NULL implementation of Serialize.
	*
	* @param	Data	unused
	* @param	Event	unused
	*/
	CORE_API virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override;
};

