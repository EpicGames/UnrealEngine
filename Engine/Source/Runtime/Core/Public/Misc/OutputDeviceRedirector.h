// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreTypes.h"
#include "Logging/LogVerbosity.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/NotNull.h"
#include "Misc/OutputDevice.h"
#include "Templates/PimplPtr.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

/*-----------------------------------------------------------------------------
FOutputDeviceRedirector.
-----------------------------------------------------------------------------*/

class FOutputDeviceFence;
namespace UE::Private { struct FOutputDeviceRedirectorState; }

/** The type of lines buffered by secondary threads. */
struct FBufferedLine
{
	TUniquePtr<TCHAR[]> Data;
	const FLazyName Category;
	const double Time;
	const ELogVerbosity::Type Verbosity;
	bool bExternalAllocation;

	CORE_API explicit FBufferedLine(const TCHAR* InData, const FName& InCategory, ELogVerbosity::Type InVerbosity, const double InTime = -1);
};

enum class EOutputDeviceRedirectorFlushOptions : uint32
{
	None = 0,

	/**
		* Flush asynchronously when possible.
		*
		* When this flag is set and there is a dedicated primary logging thread, the flush function returns immediately.
		* Otherwise, the flush function does not return until the requested type of flush is complete.
		*/
	Async = 1 << 0,
};

ENUM_CLASS_FLAGS(EOutputDeviceRedirectorFlushOptions);

/**
 * Class used for output redirection to allow logs to show in multiple output devices.
 */
class FOutputDeviceRedirector final : public FOutputDevice
{
public:
	/** Initialization constructor. */
	CORE_API FOutputDeviceRedirector();
	
	/** Destructor **/
	CORE_API ~FOutputDeviceRedirector();

	/** Get the GLog singleton. */
	static CORE_API FOutputDeviceRedirector* Get();

	/**
	 * Adds an output device to the chain of redirections.
	 *
	 * @param OutputDevice   Output device to add.
	 */
	CORE_API void AddOutputDevice(FOutputDevice* OutputDevice);

	/**
	 * Removes an output device from the chain of redirections.
	 *
	 * @param OutputDevice   Output device to remove.
	 */
	CORE_API void RemoveOutputDevice(FOutputDevice* OutputDevice);

	/**
	 * Returns whether an output device is in the list of redirections.
	 *
	 * @param OutputDevice   Output device to check the list against.
	 * @return true if messages are currently redirected to the the passed in output device, false otherwise.
	 */
	CORE_API bool IsRedirectingTo(FOutputDevice* OutputDevice);

	/** Flushes lines buffered by secondary threads. */
	CORE_API void FlushThreadedLogs(EOutputDeviceRedirectorFlushOptions Options = EOutputDeviceRedirectorFlushOptions::None);

	/**
	 * Serializes the current backlog to the specified output device.
	 * @param OutputDevice   Output device that will receive the current backlog.
	 */
	UE_DEPRECATED(5.7, "The backlog is now automatically serialized within AddOutputDevice.")
	CORE_API void SerializeBacklog(FOutputDevice* OutputDevice);

	/**
	 * Enables or disables the backlog.
	 *
	 * @param bEnable   Starts saving a backlog if true, disables and discards any backlog if false.
	 * @return true if the backlog was previously enabled or false if it was not.
	 */
	CORE_API bool EnableBacklog(bool bEnable);

	/**
	 * Sets the current thread to be the thread that redirects logs to buffered output devices.
	 *
	 * The current thread can redirect to buffered output devices without buffering, and becomes
	 * responsible for flushing buffered logs from secondary threads. Logs from secondary threads
	 * will not be redirected unless the current thread periodically flushes threaded logs.
	 */
	CORE_API void SetCurrentThreadAsPrimaryThread();

	/**
	 * Starts a dedicated primary thread that redirects logs to buffered output devices.
	 *
	 * A thread will not be started for certain configurations or platforms, or when threading is disabled.
	 *
	 * @return true if a dedicated primary logging thread is running, false otherwise.
	 */
	CORE_API bool TryStartDedicatedPrimaryThread();

	/**
	 * Serializes the log record via all current output devices.
	 *
	 * All pointers in the record must remain valid until completion of a fence created after this call.
	 */
	CORE_API void SerializeRecord(const UE::FLogRecord& Record) final;

	/**
	 * Serializes the passed in data via all current output devices.
	 *
	 * @param Data   Text to log.
	 * @param Event  Event name used for suppression purposes.
	 */
	CORE_API void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const FName& Category, const double Time) final;

	/**
	 * Serializes the passed in data via all current output devices.
	 *
	 * @param Data   Text to log.
	 * @param Event  Event name used for suppression purposes.
	 */
	CORE_API void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const FName& Category) final;

	/** Same as Serialize(). */
	CORE_API void RedirectLog(const FName& Category, ELogVerbosity::Type Verbosity, const TCHAR* Data);
	CORE_API void RedirectLog(const FLazyName& Category, ELogVerbosity::Type Verbosity, const TCHAR* Data);

	/**
	 * Creates a fence that tracks the status of async operations started prior to creation of the fence.
	 *
	 * The fence is considered complete when every log prior to the fence has been broadcast to every output device.
	 */
	CORE_API FOutputDeviceFence CreateFence();

	/** Passes on the flush request to all current output devices. */
	CORE_API void Flush() final;

	/**
	 * Attempts to set the calling thread as the panic thread and enable panic mode.
	 *
	 * Only one thread can be the panic thread. Subsequent calls from other threads are ignored.
	 * Only redirects logs to panic-safe output devices from this point forward.
	 * Makes the calling thread the primary log thread as well.
	 * Flushes buffered logs to panic-safe output devices.
	 * Flushes panic-safe output devices.
	 */
	CORE_API void Panic();

	/**
	 * Closes output device and cleans up.
	 *
	 * This can't happen in the destructor as we might have to call "delete" which cannot be done for static/global objects.
	 */
	CORE_API void TearDown() final;

	/** Determine if the backlog is enabled. */
	UE_DEPRECATED(5.7, "The backlog is now automatically serialized within AddOutputDevice.")
	CORE_API bool IsBacklogEnabled() const;

	/** Internal. Used for crash reporting. */
	UE_INTERNAL bool HasBacklogForCrash() const;
	/** Internal. Used for crash reporting. */
	UE_INTERNAL void SerializeBacklogForCrash(TNotNull<FOutputDevice*> OutputDevice) const;

private:
	TPimplPtr<UE::Private::FOutputDeviceRedirectorState> State;
};

/** Tracks pending operations on output devices. */
class FOutputDeviceFence
{
public:
	CORE_API void Wait() const;
	CORE_API bool Poll() const;

private:
	FOutputDeviceFence() = default;

	UE::Private::FOutputDeviceRedirectorState* State = nullptr;
	uint64 Index = 0;

	friend FOutputDeviceRedirector;
};
