// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/OutputDevice.h"

class FCbWriter;

class FOutputDeviceStdOutput final : public FOutputDevice
{
public:
	CORE_API FOutputDeviceStdOutput();

	bool CanBeUsedOnAnyThread() const final { return true; }
	bool CanBeUsedOnPanicThread() const final { return true; }

	void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) final
	{
		Serialize(V, Verbosity, Category, -1.0);
	}

	void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time) final;
	void SerializeRecord(const UE::FLogRecord& Record) final;

private:
	void SerializeAsText(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time);
	void SerializeRecordAsText(const UE::FLogRecord& Record);

	void SerializeAsJson(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time);
	void SerializeRecordAsJson(const UE::FLogRecord& Record);

	void WriteAsJson(const FCbWriter& Writer);

private:
	ELogVerbosity::Type AllowedLogVerbosity = ELogVerbosity::Display;
	bool bIsConsoleOutput = false;
	bool bIsJsonOutput = false;
};
