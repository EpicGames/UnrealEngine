// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Misc/Build.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"

class FExec;
class FName;
class FOutputDevice;
class FText;

/*-----------------------------------------------------------------------------
	Parsing functions.
-----------------------------------------------------------------------------*/
struct FParse
{
	/**
	 * Sees if Stream starts with the named command.  If it does,
	 * skips through the command and blanks past it.  Returns true of match.
	 * @param bParseMightTriggerExecution true: Caller guarantees this is only part of parsing and no execution happens without further parsing (good for "DumpConsoleCommands").
	 */
	static CORE_API bool Command( const TCHAR** Stream, const TCHAR* Match, bool bParseMightTriggerExecution = true );
	/** Parses a name. */
	static CORE_API bool Value(const TCHAR* Stream, const TCHAR* Match, FName& Name, const TCHAR** OptStreamGotTo = nullptr);
	/** Parses a uint32. */
	static CORE_API bool Value(const TCHAR* Stream, const TCHAR* Match, uint32& Value, const TCHAR** OptStreamGotTo = nullptr);
	/** Parses a globally unique identifier. */
	static CORE_API bool Value(const TCHAR* Stream, const TCHAR* Match, struct FGuid& Guid, const TCHAR** OptStreamGotTo = nullptr);
	/** Parses a string from a text string. 
	 * @param Stream, the string you want to extract the value from.
	 * @param Match, the identifier for the value in the stream.
	 * @param Value, the destination to the value to be extracted to.
	 * @param MaxLen, the maximum size eof the string that can be extracted.
	 * @param bShouldStopOnSeparator, (default = true) If this is true, and the value doesn't start with a '"'
				then it may be truncated to ',' or ')' in addition to whitespace.
	 * @param OptStreamGotTo, (default = nullptr) If this is not null, then its dereference is set to the address
				of the end of the value within Stream. This permits consuming of stream in a loop where Match may
				occur multiple times.
	*/
	static CORE_API bool Value(const TCHAR* Stream, const TCHAR* Match, TCHAR* Value, int32 MaxLen, bool bShouldStopOnSeparator=true, const TCHAR** OptStreamGotTo = nullptr);
	/** Parses a byte. */
	static CORE_API bool Value(const TCHAR* Stream, const TCHAR* Match, uint8& Value, const TCHAR** OptStreamGotTo = nullptr);
	/** Parses a signed byte. */
	static CORE_API bool Value(const TCHAR* Stream, const TCHAR* Match, int8& Value, const TCHAR** OptStreamGotTo = nullptr);
	/** Parses a uint16. */
	static CORE_API bool Value(const TCHAR* Stream, const TCHAR* Match, uint16& Value, const TCHAR** OptStreamGotTo = nullptr);
	/** Parses a signed word. */
	static CORE_API bool Value(const TCHAR* Stream, const TCHAR* Match, int16& Value, const TCHAR** OptStreamGotTo = nullptr);
	/** Parses a floating-point value. */
	static CORE_API bool Value(const TCHAR* Stream, const TCHAR* Match, float& Value, const TCHAR** OptStreamGotTo = nullptr);
	/** Parses a double precision floating-point value. */
	static CORE_API bool Value(const TCHAR* Stream, const TCHAR* Match, double& Value, const TCHAR** OptStreamGotTo = nullptr);
	/** Parses a signed double word. */
	static CORE_API bool Value(const TCHAR* Stream, const TCHAR* Match, int32& Value, const TCHAR** OptStreamGotTo = nullptr);
	/** Parses a string. 
	 * @param Stream, the string you want to extract the value from.
	 * @param Match, the identifier for the value in the stream.
	 * @param Value, the destination to the value to be extracted to.
	 * @param bShouldStopOnSeparator, (default = true) If this is true, and the value doesn't start with a '"'
				then it may be truncated to ',' or ')' in addition to whitespace.
	 * @param OptStreamGotTo, (default = nullptr) If this is not null, then its dereference is set to the address
				of the end of the value within Stream. This permits consuming of stream in a loop where Match may
				occur multiple times.
	*/
	static CORE_API bool Value(const TCHAR* Stream, const TCHAR* Match, FString& Value, bool bShouldStopOnSeparator =true, const TCHAR** OptStreamGotTo = nullptr);
	/** Parses an FText. Doesn't support OptStreamGotTo like the other Value versions due to FText parsing
	 * not returning the end of the text
	 * @param Stream, the string you want to extract the value from.
	 * @param Match, the identifier for the value in the stream.
	 * @param Value, the destination FText value to be extracted to.
	 * @param Namespace, an optional namespace to use when parsing texts that use LOCTEXT (default is an empty namespace).
	*/
	static CORE_API bool Value(const TCHAR* Stream, const TCHAR* Match, FText& Value, const TCHAR* Namespace = NULL);
	/** Parses a quadword. */
	static CORE_API bool Value(const TCHAR* Stream, const TCHAR* Match, uint64& Value, const TCHAR** OptStreamGotTo = nullptr);
	/** Parses a signed quadword. */
	static CORE_API bool Value(const TCHAR* Stream, const TCHAR* Match, int64& Value, const TCHAR** OptStreamGotTo = nullptr);
	/** Parses a boolean value. */
	static CORE_API bool Bool(const TCHAR* Stream, const TCHAR* Match, bool& OnOff, const TCHAR** OptStreamGotTo = nullptr);
	/** Get a line of Stream (everything up to, but not including, CR/LF. Returns 0 if ok, nonzero if at end of stream and returned 0-length string. */
	static CORE_API bool Line( const TCHAR** Stream, TCHAR* Result, int32 MaxLen, bool Exact= false );
	/** Get a line of Stream (everything up to, but not including, CR/LF. Returns 0 if ok, nonzero if at end of stream and returned 0-length string. */
	static CORE_API bool Line( const TCHAR** Stream, FString& Result, bool Exact = false );
	/** Get a line of Stream (everything up to, but not including, CR/LF. Returns 0 if ok, nonzero if at end of stream and returned 0-length string. */
	static CORE_API bool Line( const TCHAR** Stream, FStringView& Result, bool Exact = false );
	/** Get a line of Stream, with support for extending beyond that line with certain characters, e.g. {} and \
	 * the out character array will not include the ignored endlines
	 */
	enum class ELineExtendedFlags : uint8
	{
		None = 0,
		SwallowDoubleSlashComments = 1,
		SwallowSemicolonComments = 2,
		SwallowExtraEOLs = 4,
		BreakOnPipe = 8,
		AllowBracketedMultiline = 16, // if a EOL happens between {} curly brackets, insert a space, and continue parsing the next line
		AllowEscapedEOLMultiline = 32, // if a EOL happens right after a \ backslash, insert a space, and continue parsing the next line
		
		// Standard maps to an "expected" set of flags (\<eol> is replaced with space, no comments, don't swallow extra EOLs, no { } processing)
		Standard = AllowEscapedEOLMultiline,

		// OldDefaultMode maps to what the previous 'bExact' mapped to when false
		OldDefaultMode = SwallowDoubleSlashComments | BreakOnPipe | AllowBracketedMultiline | AllowEscapedEOLMultiline | SwallowExtraEOLs,
		// OldExactMode maps to what the previous 'bExact' mapped to when true
		OldExactMode = AllowBracketedMultiline | AllowEscapedEOLMultiline,
	};

	static CORE_API bool LineExtended(const TCHAR** Stream, FString& Result, int32& LinesConsumed, ELineExtendedFlags Flags = ELineExtendedFlags::Standard);
	static CORE_API bool LineExtended(const TCHAR** Stream, FStringBuilderBase& Result, int32& LinesConsumed, ELineExtendedFlags Flags = ELineExtendedFlags::Standard);
	/** Grabs the next space-delimited (unless SingleCharacterDelimiter is provided) string from the input stream. If quoted, gets entire quoted string. */
	static CORE_API bool Token( const TCHAR*& Str, TCHAR* Result, int32 MaxLen, bool bUseEscape, const TCHAR SingleCharacterDelimiter = TEXT('\0'));
	/** Grabs the next space-delimited (unless SingleCharacterDelimiter is provided) string from the input stream. If quoted, gets entire quoted string. */
	static CORE_API bool Token( const TCHAR*& Str, FString& Arg, bool bUseEscape, const TCHAR SingleCharacterDelimiter = TEXT('\0'));
	/** Grabs the next alpha-numeric space-delimited token from the input stream. */
	static CORE_API bool AlnumToken(const TCHAR*& Str, FString& Arg);
	/** Grabs the next space-delimited string from the input stream. If quoted, gets entire quoted string. */
	static CORE_API FString Token( const TCHAR*& Str, bool UseEscape );
	/** Get next command.  Skips past comments and cr's. */
	static CORE_API void Next( const TCHAR** Stream );
	/** Checks if a command-line parameter exists in the stream. */
	static CORE_API bool Param( const TCHAR* Stream, const TCHAR* Param );
	/** Parse an Text token. */
	static CORE_API bool Text( const TCHAR* Stream, FText& Value, const TCHAR* Namespace = nullptr );
	/** Parse a quoted string token. */
	static CORE_API bool QuotedString( const TCHAR* Stream, FString& Value, int32* OutNumCharsRead = nullptr );
	static CORE_API bool QuotedString( const TCHAR* Stream, FStringBuilderBase& Value, int32* OutNumCharsRead = nullptr );
	/**
	* Parse the next space-delimited (or SingleCharacterDelmiter-delimited) expression, using brackets
	* (), {}, or [] to allow subexpressions to be part of the single expression even if they contain the
	* delimiter.
	* The parsed expression can be either an atomic token such as "Value1" in the stream "Value1 Value2 Value3"
	* or a bracketed expression such as "(Value1.1 Value1.2)" in the stream "(Value1.1 Value1.2) Value2 Value3".
	* If quotes are encountered, text inside of the quotes is not interpreted (brackets and delimiters are ignored)
	* and it is included verbatim in the parsed expression.
	* Trailing delimiters are not consumed, only leading delimiters.
	*/
	static CORE_API bool Expression(const TCHAR*& Str, FString& OutExpression, bool bUseEscape, const TCHAR SingleCharacterDelimiter = TEXT('\0'));

	//
	// Parse a hex digit.
	//
	static inline int32 HexDigit(TCHAR c)
	{
		int32 Result = 0;

		if (c >= '0' && c <= '9')
		{
			Result = c - '0';
		}
		else if (c >= 'a' && c <= 'f')
		{
			Result = c + 10 - 'a';
		}
		else if (c >= 'A' && c <= 'F')
		{
			Result = c + 10 - 'A';
		}
		else
		{
			Result = 0;
		}

		return Result;
	}

	/** Parses a hexadecimal string value. */
	static CORE_API uint32 HexNumber(FStringView HexString);
	static CORE_API uint64 HexNumber64(FStringView HexString);

	/** Parses the scheme name from a URI */
	static CORE_API bool SchemeNameFromURI(const TCHAR* InURI, FString& OutSchemeName);


	//
	// CLI string parsing using grammar based parser.
	//

	enum class EGrammarBasedParseFlags
	{
		None				= 0u,
		AllowQuotedCommands = 1 << 0u,
	};

	enum class EGrammarBasedParseErrorCode
	{
		Succeeded,
		NotRun,
		UnBalancedQuote,
		DisallowedQuotedCommand
	};

	struct FGrammarBasedParseResult
	{
		const TCHAR* At = nullptr;
		EGrammarBasedParseErrorCode ErrorCode = EGrammarBasedParseErrorCode::NotRun;
	};

	/** Grammar
	* 	 Line  -> Cmd*
	* 	 Cmd   -> "Cmd*"			-- allowed if EGrammaredParseFlags::AllowQuotedCommands is given
	* 	 	   | Key(=Value)?   -- invokes OnCommandCallback
	* 	 Key   -> (/|(-?-?))Ident
	* 	 Value -> "[^"]*"
	* 	 	   | [^" \t\r\n]+
	* 	 Ident -> [_a-zA-Z][_a-zA-Z0-9.]*
	* 
	* Grammar Key
	*   Expressions
	*  
	*   Operators
	*    * = 0 or more of a expression
	*    + = 1 or more of a expression
	*    ? = 0 or 1 of an expression (IE, its optional)
	*   [] = set of characters
	*   () = treat enclosed expressions as 1 for purpose of other operators
	*/
	static CORE_API FGrammarBasedParseResult GrammarBasedCLIParse(const TCHAR* Stream, TFunctionRef<void(FStringView, FStringView)> OnCommandCallback, EGrammarBasedParseFlags Flags = EGrammarBasedParseFlags::AllowQuotedCommands);
};

ENUM_CLASS_FLAGS(FParse::EGrammarBasedParseFlags);
ENUM_CLASS_FLAGS(FParse::ELineExtendedFlags)

#if !UE_BUILD_SHIPPING

UE_DEPRECATED("5.6", "Use the overload that takes the entire command instead of a pattern.")
CORE_API bool ConsoleCommandLibrary_DumpLibrary(class UWorld* InWorld, FExec& SubSystem, const FString& Cmd, FOutputDevice& Ar);

/** Needed for the console command "DumpConsoleCommands". Outputs registered and unregistered console commands to the supplied device. */
CORE_API bool ConsoleCommandLibrary_DumpLibrary(class UWorld* InWorld, FExec& SubSystem, const TCHAR* Cmd, FOutputDevice& Ar);

/** Needed for the console command "Help". Outputs registered and unregistered console commands and console variables the supplied file path. */
CORE_API bool ConsoleCommandLibrary_DumpLibraryHTML(class UWorld* InWorld, FExec& SubSystem, const FString& OutPath);
#endif // !UE_BUILD_SHIPPING
