// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

namespace UE::Paths
{

/** Flags used for argument passing in FPaths and FPathViews functions. */
enum class EFlags : uint64
{
	None = 0,

	/** In functions that return extensions, request that the extension include the dot. */
	IncludeDot = 1 << 0,

	/**
	 * In functions that split extensions out of a path, allow the extension to include multiple dots,
	 * after the final directory separator. For example:
	 * "/Path/PackageName.m.uasset" ->
	 *    (Compound Allowed)      "/Path", "PackageName", ".m.uasset"
	 *    (Compound not allowed)  "/Path", "PackageName.m", ".uasset"
	 */
	AllowCompoundExtension = 1 << 1,
};
ENUM_CLASS_FLAGS(EFlags);

} // namespace UE::Paths

