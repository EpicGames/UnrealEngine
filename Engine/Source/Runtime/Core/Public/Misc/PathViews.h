// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "CoreTypes.h"
#include "Misc/PathTypes.h"

class FName;
class FString;
template <typename CharType> class TStringBuilderBase;
template <typename FuncType> class TFunctionRef;

class FPathViews
{
public:
	/**
	 * Returns the portion of the path after the last separator.
	 *
	 * Examples: (Using '/' but '\' is valid too.)
	 * "A/B/C.D" -> "C.D"
	 * "A/B/C"   -> "C"
	 * "A/B/"    -> ""
	 * "A"       -> "A"
	 *
	 * @return The portion of the path after the last separator.
	 */
	static CORE_API FStringView GetCleanFilename(const FStringView& InPath);

	/**
	 * Returns the portion of the path after the last separator and before the last dot.
	 *
	 * Examples: (Using '/' but '\' is valid too.)
	 * "A/B/C.D" -> "C"
	 * "A/B/C"   -> "C"
	 * "A/B/"    -> ""
	 * "A"       -> "A"
	 *
	 * @return The portion of the path after the last separator and before the last dot.
	 */
	static CORE_API FStringView GetBaseFilename(const FStringView& InPath);

	/**
	 * Returns the portion of the path before the last dot.
	 *
	 * Examples: (Using '/' but '\' is valid too.)
	 * "A/B/C.D" -> "A/B/C"
	 * "A/B/C"   -> "A/B/C"
	 * "A/B/"    -> "A/B/"
	 * "A"       -> "A"
	 *
	 * @return The portion of the path before the last dot.
	 */
	static CORE_API FStringView GetBaseFilenameWithPath(const FStringView& InPath);

	/**
	 * Returns the portion of the path before the last separator.
	 *
	 * Examples: (Using '/' but '\' is valid too.)
	 * "A/B/C.D" -> "A/B"
	 * "A/B/C"   -> "A/B"
	 * "A/B/"    -> "A/B"
	 * "A"       -> ""
	 *
	 * @return The portion of the path before the last separator.
	 */
	static CORE_API FStringView GetPath(const FStringView& InPath);

	/**
	 * Returns the portion of the path after the last dot following the last separator,
	 * optionally including the dot, optionally allowing compound extensions.
	 * 
	 * Examples: (Using '/' but '\' is valid too.)
	 * "A/B.C.D" -> "D"    (EFlags::IncludeDot == 0, EFlags::AllowCompoundExtension == 0)
	 *           -> ".D"   (EFlags::IncludeDot == 1, EFlags::AllowCompoundExtension == 0)
	 *           -> "C.D"  (EFlags::IncludeDot == 0, EFlags::AllowCompoundExtension == 1)
	 *           -> ".C.D" (EFlags::IncludeDot == 1, EFlags::AllowCompoundExtension == 1)
	 * "A/B/C.D" -> "D" (EFlags::IncludeDot == 0) or ".D" (EFlags::IncludeDot == 1)
	 * "A/B/.D"  -> "D" (EFlags::IncludeDot == 0) or ".D" (EFlags::IncludeDot == 1)
	 * ".D"      -> "D" (EFlags::IncludeDot == 0) or ".D" (EFlags::IncludeDot == 1)
	 * "A/B/C"   -> ""
	 * "A.B/C"   -> ""
	 * "A.B/"    -> ""
	 * "A"       -> ""
	 * "A.B/C.D" -> "D" (EFlags::IncludeDot) or ".D" (EFlags::IncludeDot), even if EFlags::AllowCompoundExtension == 1.
	 *
	 * Default: Dot is not included, compound extensions are not allowed.
	 * @param Flags
	 *		EFlags::IncludeDot Includes the leading dot in the result.
	 *		EFlags::AllowCompoundExtension Allows compound extensions (.a.b) in the result.
	 *
	 * @return The portion of the path after the last dot following the last separator, optionally including the dot.
	 */
	static CORE_API FStringView GetExtension(const FStringView& InPath,
		UE::Paths::EFlags Flags = UE::Paths::EFlags::None);
	/** Legacy version of GetExtension that uses a bool to specify bIncludeDot. EFlags::AllowCompoundExtension == 0. */
	static CORE_API FStringView GetExtension(const FStringView& InPath, bool bIncludeDot);

	/**
	 * Returns the last non-empty path component.
	 *
	 * Examples: (Using '/' but '\' is valid too.)
	 * "A/B/C.D" -> "C.D"
	 * "A/B/C"   -> "C"
	 * "A/B/"    -> "B"
	 * "A"       -> "A"
	 *
	 * @return The last non-empty path component.
	 */
	static CORE_API FStringView GetPathLeaf(const FStringView& InPath);

	/**
	 * Return whether the given relative or absolute path is a leaf path - has no separators.
	 * 
	 * Examples: (Using '/' but '\' functions the same way)
	 * A		-> true
	 * A/		-> true
	 * D:/		-> true
	 * /		-> true
	 * //		-> true
	 * A/B		-> false
	 * D:/A		-> false
	 * //A		-> false
	 */
	static CORE_API bool IsPathLeaf(FStringView InPath);

	/**
	 * Report whether the given path is an invalid path because it has a drive specifier (':') without a following
	 * path separator indicating the root of that drive. In general usage on Windows, drive specifiers without a
	 * following path separator are interpreted to mean the current working directory of the given drive. But that
	 * context is not always applicable and will need to be handled in system-specific ways.
	 * @see SplitVolumeSpecifier for callers who want to followup by modifing the path.
	 * 
	 * D:		-> true
	 * D:/		-> false
	 * D:root	-> true
	 * D:/root	-> false
	 * //volume -> false
	 * /root    -> false
	 * relpath  -> false
	 * 
	 * See FPathViewsVolumeSpecifierTest for further edgecases.
	 */
	static CORE_API bool IsDriveSpecifierWithoutRoot(FStringView InPath);

	/**
	 * Splits InPath into individual directory components, and calls ComponentVisitor on each.
	 * Note that this function does not properly handle Windows network paths such as //ShareName/Path and
	 * drive components such as C: (without trailing slash) may not be considered valid paths by some APIs.
	 *
	 * Examples:
	 * "A/B.C" -> {"A", "B.C"}
	 * "A/B/C" -> {"A", "B", "C"}
	 * "../../A/B/C.D" -> {"..", "..", "A", "B", "C.D" }
	 * "/A/B/C" -> { "", "A", "B", "C" }
	 * "C:/A/B/C" -> { "C:", "A", "B", "C" }
	 * "//ShareName/Path" -> { "", "", "ShareName", "Path" }
	 */
	static CORE_API void IterateComponents(FStringView InPath, TFunctionRef<void(FStringView)> ComponentVisitor);

	/**
	 * Calls AncestorVisitor with the given path and its ancestor paths.
	 * If AncestorVisitor returns false, iteration stops.
	 * If the string begins with a path root as understood by SplitVolumeSpecifier, that will be the last item visited
	 * with the modification that a separator following a drive specifier will be included by this function.
	 *
	 * Examples:
	 * "" -> { "" }
	 * "/" -> { "/" }
	 * "A/B.C" -> { "A/B.C", "A" }
	 * "A/B/C" -> { "A/B/C", "A/B", "A" }
	 * "../../A/B/C.D" -> { "../../A/B/C.D", "../../A/B", "../../A", "../..", ".." }
	 * "/A/B" -> { "/A/B", "/A", "/" }
	 * "/A/B/" -> { "/A/B", "/A", "/" }
	 * "C:/A/B" -> { "C:/A/B", "C:/A", "C:/" }
	 * "C:A/B" -> { "C:A/B", "C:A", "C:" }
	 * "\\ShareName/A/B" -> { "\\ShareName/A/B", "\\ShareName/A", "\\ShareName" }
	 *
	 * When moving to the next ancestor, extra trailing separators are also removed, e.g.
	 * "/A//B" -> { "/A//B", "/A" }
	 * Note that separators are not removed except at the end of the current path, as with visiting "/A//B" above.
	 */
	static CORE_API void IterateAncestors(FStringView InPath, TFunctionRef<bool(FStringView)> AncestorVisitor);

	/**
	 * Splits a path into three parts, any of which may be empty: the path, the clean name, and the extension.
	 *
	 * Examples: (Using '/' but '\' is valid too.)
	 * "A/B/C.D" -> ("A/B", "C",   "D")
	 * "A/B/C"   -> ("A/B", "C",   "")
	 * "A/B/"    -> ("A/B", "",    "")
	 * "A/B/.D"  -> ("A/B", "",    "D")
	 * "A/B.C.D" -> ("A",   "B.C", "D")
	 * "A"       -> ("",    "A",   "")
	 * "A.D"     -> ("",    "A",   "D")
	 * ".D"      -> ("",    "",    "D")
	 *
	 * @param OutPath [out] Receives the path portion of the input string, excluding the trailing separator.
	 * @param OutName [out] Receives the name portion of the input string.
	 * @param OutExt  [out] Receives the extension portion of the input string, excluding the dot.
	 */
	static CORE_API void Split(const FStringView& InPath, FStringView& OutPath, FStringView& OutName, FStringView& OutExt);

	/**
	 * Appends each suffix argument to the path in the builder and ensures that there is a separator between them.
	 *
	 * Examples:
	 * ("",    "")    -> ""
	 * ("A",   "")    -> "A/"
	 * ("",    "B")   -> "B"
	 * ("/",   "B")   -> "/B"
	 * ("A",   "B")   -> "A/B"
	 * ("A/",  "B")   -> "A/B"
	 * ("A\\", "B")   -> "A\\B"
	 * ("A/B", "C/D") -> "A/B/C/D"
	 * ("A/", "B", "C/", "D") -> "A/B/C/D"
	 *
	 * @param Builder A possibly-empty path that may end in a separator.
	 * @param Args Arguments that can write to a string builder and do not start with a separator.
	 */
	template <typename CharType, typename... ArgTypes>
	static void Append(TStringBuilderBase<CharType>& Builder, ArgTypes&&... Args)
	{
		const auto AddSeparator = [&Builder]() -> TStringBuilderBase<CharType>&
		{
			if (!(Builder.Len() == 0 || Builder.LastChar() == '/' || Builder.LastChar() == '\\'))
			{
				Builder.AppendChar('/');
			}
			return Builder;
		};
		((AddSeparator() << Args), ...);
	}

	/**
	 * Replaces the pre-existing file extension of a filename. 
	 *
	 * @param InPath A valid file path with a pre-existing extension.
	 * @param InNewExtension The new extension to use (prefixing with a '.' is optional)
	 *
	 * @return The new file path complete with the new extension unless InPath is not valid in which
	 * case a copy of InPath will be returned instead.
	 */
	static CORE_API FString ChangeExtension(const FStringView& InPath, const FStringView& InNewExtension);
	
	/**
	 * Sets the file extension of a filename.
	 *
	 * @param InPath A file path with or without a pre-existing extension.
	 * @param InNewExtension The new extension to use (prefixing with a '.' is optional)
	 *
	 * @return The new file path complete with the new extension.
	 */
	static CORE_API FString SetExtension(const FStringView& InPath, const FStringView& InNewExtension);

	/** Return whether the given character is a path-separator character (/ or \) */
	static CORE_API bool IsSeparator(TCHAR c);

	/**
	 * Return true if the given paths are the same path (with exceptions noted below).
	 * Case-insensitive
	 * / is treated as equal to \
	 * Presence or absence of terminating separator (/) is ignored in the comparison.
	 * Directory elements of . and .. are currently not interpreted and are treated as literal characters.
	 *    Callers should not rely on this behavior as it may be corrected in the future.
	 *    callers should instead conform the paths before calling.
	 * Relative paths and absolute paths are not resolved, and relative paths will never equal absolute paths.
	 *    Callers should not rely on this behavior as it may be corrected in the future;
	 *    callers should instead conform the paths before calling.
	 * Examples:
	 * ("../A/B.C", "../A/B.C") -> true
	 * ("../A/B", "../A/B.C")	-> false
	 * ("../A/", "../A/")		-> true
	 * ("../A/", "../A")		-> true
	 * ("d:/root/Engine/", "d:\root\Engine") -> true
	 * (../../../Engine/Content", "d:/root/Engine/Content") -> false
	 * (d:/root/Engine/..", "d:/root") -> false
	 * (d:/root/Engine/./Content", "d:/root/Engine/Content") -> false
	 */
	static CORE_API bool Equals(FStringView A, FStringView B);

	/**
	 * Return true if the the first path is lexicographically less than the second path (with caveats noted below).
	 * Case-insensitive
	 * / is treated as equal to \
	 * Presence or absence of terminating separator (/) is ignored in the comparison.
	 * Directory elements of . and .. are currently not interpreted and are treated as literal characters.
	 *    Callers should not rely on this behavior as it may be corrected in the future.
	 *    callers should instead conform the paths before calling.
	 * Relative paths and absolute paths are not resolved, and relative paths will never equal absolute paths.
	 *    Callers should not rely on this behavior as it may be corrected in the future;
	 *    callers should instead conform the paths before calling.
	 * Examples:
	 * ("../A/B.C", "../A/B.C") -> false (they are equal)
	 * ("../A/B", "../A/B.C")	-> true (a string is greater than any prefix of itself)
	 * ("../A/", "../A/")		-> false (they are equal)
	 * ("../A/", "../A")		-> false (they are equal)
	 * ("../A", "../A/")		-> false (they are equal)
	 * ("d:/root/Engine/", "d:\root\Engine") -> false (they are equal)
	 * (../../../Engine/Content", "d:/root/Engine/Content") -> true ('.' is less than 'd')
	 * (d:/root/Engine/..", "d:/root") -> false (A string is greater than any prefix of itself)
	 * (d:/root/Engine/./Content", "d:/root/Engine/Content") -> false
	 */
	static CORE_API bool Less(FStringView A, FStringView B);

	/**
	 * Check whether Parent is a parent path of Child and report the relative path if so.
	 * Case-insensitive
	 * / is treated as equal to \
	 * Presence or absence of terminating separator (/) is ignored in the comparison.
	 * Directory elements of . and .. are currently not interpreted and are treated as literal characters.
	 *    Callers should not rely on this behavior as it may be corrected in the future.
	 *    callers should instead conform the paths before calling.
	 * Relative paths and absolute paths are not resolved, and relative paths will never equal absolute paths.
	 *    Callers should not rely on this behavior as it may be corrected in the future;
	 *    callers should instead conform the paths before calling.
	 * Examples:
	 * ("../A/B", "../A")	-> (true, "B")
	 * ("../A\B", "../A/")	-> (true, "B")
	 * ("../A/", "../A")	-> (true, "")
	 * (".././A/", "../A")	-> (false, "")
	 * ("../../../Engine", "d:/root/Engine") -> (false, "")
	 *
	 * @param Child An absolute path that may be a child path of Parent.
	 * @param Parent An absolute path that may be a parent path of Child.
	 * @param OutRelPath Receives the relative path from Parent to Child, or empty if Parent is not a parent of Child.
	 *
	 * @return True if and only if Child is a child path of Parent (or is equal to it).
	 */
	static CORE_API bool TryMakeChildPathRelativeTo(FStringView Child, FStringView Parent, FStringView& OutRelPath);

	/**
	 * Return whether Parent is a parent path of (or is equal to) Child.
	 * Case-insensitive
	 * / is treated as equal to \
	 * Presence or absence of terminating separator (/) is ignored in the comparison.
	 * Directory elements of . and .. are currently not interpreted and are treated as literal characters.
	 *    Callers should not rely on this behavior as it may be corrected in the future.
	 *    callers should instead conform the paths before calling.
	 * Relative paths and absolute paths are not resolved, and relative paths will never equal absolute paths.
	 *    Callers should not rely on this behavior as it may be corrected in the future;
	 *    callers should instead conform the paths before calling.
	 * Examples:
	 * ("../A", "../A/B")	-> true
	 * ("../A/", "../A\B")	-> true
	 * ("../A", "../A/")	-> true
	 * ("../A", ".././A/")	-> false
	 * ("d:/root/Engine", "../../../Engine") -> false
	 *
	 * @param Parent An absolute path that may be a parent path of Child.
	 * @param Child An absolute path that may be a child path of Parent.
	 *
	 * @return True if and only if Child is a child path of Parent (or is equal to it).
	 */
	static CORE_API bool IsParentPathOf(FStringView Parent, FStringView Child);

	/**
	 * Return whether the given path is a relativepath - does not start with a separator or volume:.
	 * Returns true for empty paths.
	 */
	static CORE_API bool IsRelativePath(FStringView InPath);

	/**
	 * Report whether the path has an unneeded trailing slash. 
	 * /root/path	-> false
	 * /root/		-> true
	 * /root//		-> true
	 * /			-> false
	 * //			-> false
	 * ///			-> true
	 * d:/			-> false
	 * d://			-> true
	 */
	static CORE_API bool HasRedundantTerminatingSeparator(FStringView A);

	/** Convert to absolute using process BaseDir(), normalize and append. FPaths::ConvertRelativePathToFull() equivalent. */
	static CORE_API void ToAbsolutePath(FStringView InPath, FStringBuilderBase& OutPath);
	
	/** Convert to absolute using explicit BasePath, normalize and append. FPaths::ConvertRelativePathToFull() equivalent. */
	static CORE_API void ToAbsolutePath(FStringView BasePath, FStringView InPath, FStringBuilderBase& OutPath);
	
	/** Convert to absolute using process BaseDir() and normalize inlined. FPaths::ConvertRelativePathToFull() equivalent. */
	static CORE_API void ToAbsolutePathInline(FStringBuilderBase& InOutPath);
	
	/** Convert to absolute using explicit BasePath and normalize inlined. FPaths::ConvertRelativePathToFull() equivalent. */
	static CORE_API void ToAbsolutePathInline(FStringView BasePath, FStringBuilderBase& InOutPath);

	/** Convert \\ to / and do platform-specific normalization */
	static CORE_API void NormalizeFilename(FStringBuilderBase& InOutPath);

	/** Normalize and remove trailing slash unless the preceding character is '/' or ':' */
	static CORE_API void NormalizeDirectoryName(FStringBuilderBase& InOutPath);

	/** Collapses redundant paths like "/./" and "SkipDir/..". FPaths::CollapseRelativeDirectories() equivalent. */
	static CORE_API bool CollapseRelativeDirectories(FStringBuilderBase& InOutPath, bool bCollapseAllPossible = false);

	/** Removes duplicate forward slashes, e.g. "a/b//c////f.e" -> "a/b/c/f.e" */
	static CORE_API void RemoveDuplicateSlashes(FStringBuilderBase& InOutPath);

	/**
	 * Split the given absolute or relative path into its topmost directory and the relative path from that directory.
	 * Directory elements of . and .. are currently not interpreted and are treated as literal characters.
	 *    Callers should not rely on this behavior as it may be corrected in the future.
	 *    callers should instead conform the paths before calling.
	 *
	 * @param InPath The path to split.
	 * @param OutFirstComponent Receives the first directory element in the path, or InPath if it is a leaf path.
	 * @param OutRemainder Receives the relative path from OutFirstComponent to InPath, or empty if InPath is a leaf path.
	 */
	static CORE_API void SplitFirstComponent(FStringView InPath, FStringView& OutFirstComponent, FStringView& OutRemainder);

	/**
	 * Split the path into a volume specifier and the rest of the path.
	 * This function handles drive specifiers without a root in a specific way, see the examples and
	 * @see IsDriveSpecifierWithoutRoot
	 *
	 * <emptystring>      -> { '', '' }
	 * D:		          -> { 'D:', '' }
	 * D:root/path        -> { 'D:', 'root/path' }
	 * D:/root/path       -> { 'D:', '/root/path' }
	 * //volume           -> { '//volume', '' }
	 * //volume/root/path -> { '//volume', '/root/path' }
	 * root/path          -> { '', 'root/path' }
	 * /root/path         -> { '', '/root/path' }
	 * 
	 * See FPathViewsVolumeSpecifierTest for further edgecases.
	 */
	static CORE_API void SplitVolumeSpecifier(FStringView InPath, FStringView& OutVolumeSpecifier, FStringView& OutRemainder);

	/**
	 * If AppendPath is a relative path, append it as a relative path onto InOutPath.
	 * If AppendPath is absolute, reset InOutPath and replace it with RelPath.
	 * Handles presence or absence of terminating separator in BasePath.
	 * Does not interpret . or ..; each occurrence of these in either path will remain in the combined InOutPath.
	 */
	static CORE_API void AppendPath(FStringBuilderBase& InOutPath, FStringView AppendPath);

	/**
	 * Returns the name of the mount point in a path
	 * Removes starting forward slash and Classes_ prefix if bInWithoutSlashes is true
	 * Example: "/Classes_A/Textures" returns "A" and sets bOutHadClassesPrefix=true if bInWithoutSlashes is true
	 *           returns "/Classes_A" otherwise and sets bOutHadClassesPrefix=false
	 */
	static CORE_API FStringView GetMountPointNameFromPath(const FStringView InPath, bool* bOutHadClassesPrefix = nullptr, bool bInWithoutSlashes = true);
};
