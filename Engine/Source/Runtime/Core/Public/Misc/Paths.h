// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "Misc/PathTypes.h"
#include "Templates/UnrealTemplate.h"

#include <type_traits>

class FText;

namespace UE::Core::Private
{
	const TCHAR*   GetBindingType(const TCHAR* Ptr);
	const FString& GetBindingType(const FString& Str);
	const FStringView& GetBindingType(const FStringView& StringView);

	// This is used to force the arguments of FPaths::Combine to implicitly convert (if necessary)
	// to FString when calling CombineImpl(), allowing them to remain as temporaries on the stack
	// so that they stay allocated during the combining process.
	//
	// Pointer/view arguments are passed without causing FString temporaries to be created,
	// and FString arguments are referenced directly without creating extra copies.
	template <typename T>
	using TToStringType_T = decltype(GetBindingType(std::declval<T>()));
}

/**
 * Path helpers for retrieving game dir, engine dir, etc.
 */
class FPaths
{
public:

	/**
	 * @return true if ProjectDir() is allowed to be called
	 */
	static CORE_API bool CanGetProjectDir();

	/**
	  * @return true if the build has been staged and uses the simplified remapped directory structure made when staging
	 */
	static CORE_API bool IsStaged();
	
	/**
	 * Should the "saved" directory structures be rooted in the user dir or relative to the "engine/game" 
	 */
	static CORE_API bool ShouldSaveToUserDir();

	/**
	 * Returns the directory the application was launched from (useful for commandline utilities)
	 */
	static CORE_API FString LaunchDir();
	 
	/** 
	 * Returns the base directory of the "core" engine that can be shared across
	 * several games or across games & mods. Shaders and base localization files
	 * e.g. reside in the engine directory.
	 *
	 * @return engine directory
	 */
	static CORE_API FString EngineDir();

	/**
	* Returns the root directory for user-specific engine files. Always writable.
	*
	* @return root user directory
	*/
	static CORE_API FString EngineUserDir();

	/**
	* Returns the root directory for user-specific engine files which can be shared between versions. Always writable.
	*
	* @return root user directory
	*/
	static CORE_API FString EngineVersionAgnosticUserDir();

	/** 
	 * Returns the content directory of the "core" engine that can be shared across
	 * several games or across games & mods. 
	 *
	 * @return engine content directory
	 */
	static CORE_API FString EngineContentDir();

	/**
	 * Returns the directory the root configuration files are located.
	 *
	 * @return root config directory
	 */
	static CORE_API FString EngineConfigDir();

	/**
	 * Returns the Editor Settings directory of the engine
	 *
	 * @return Editor Settings directory.
	 */
	static CORE_API FString EngineEditorSettingsDir();

	/**
	 * Returns the intermediate directory of the engine
	 *
	 * @return content directory
	 */
	static CORE_API FString EngineIntermediateDir();

	/**
	 * Returns the saved directory of the engine
	 *
	 * @return Saved directory.
	 */
	static CORE_API FString EngineSavedDir();

	/**
	 * Returns the plugins directory of the engine
	 *
	 * @return Plugins directory.
	 */
	static CORE_API FString EnginePluginsDir();

	/**
	 * Returns the directory for default Editor UI Layout files of the engine
	 * @return Directory for default Editor UI Layout files.
	 */
	static CORE_API FString EngineDefaultLayoutDir();

	/**
	 * Returns the directory for project Editor UI Layout files of the engine
	 * @return Directory for project Editor UI Layout files.
	 */
	static CORE_API FString EngineProjectLayoutDir();

	/**
	 * Returns the directory for user-generated Editor UI Layout files of the engine
	 * @return Directory for user-generated Editor UI Layout files.
	 */
	static CORE_API FString EngineUserLayoutDir();

	/** 
	* Returns the base directory enterprise directory.
	*
	* @return enterprise directory
	*/
	static CORE_API FString EnterpriseDir();

	/**
	* Returns the enterprise plugins directory
	*
	* @return Plugins directory.
	*/
	static CORE_API FString EnterprisePluginsDir();

	/**
	* Returns the enterprise FeaturePack directory
	*
	* @return FeaturePack directory.
	*/
	static CORE_API FString EnterpriseFeaturePackDir();

	/**
	 * Returns the directory where the engine's platform extensions resides for the given platform
	 *
	 * @param Platform the platform to get the extension directory for
	 * @return engine's platform extension directory
	 */
	static FString EnginePlatformExtensionDir(const TCHAR* Platform)
	{
		return ConvertPath(FPaths::EngineDir(), EPathConversion::Engine_PlatformExtension, Platform);
	}

	/**
	 * Returns the directory where the project's platform extensions resides for the given platform
	 *
	 * @param Platform the platform to get the extension directory for
	 * @return project's platform extension directory
	 */
	static FString ProjectPlatformExtensionDir(const TCHAR* Platform)
	{
		return ConvertPath(FPaths::ProjectDir(), EPathConversion::Project_PlatformExtension, Platform);
	}
	
	enum class EPathConversion : uint8
	{
		// ExtraData is name of platform
		Engine_PlatformExtension,
		Engine_NotForLicensees,
		Engine_NoRedist,
		Engine_LimitedAccess,

		Project_First,
		Project_PlatformExtension = Project_First,
		Project_NotForLicensees,
		Project_NoRedist,
		Project_LimitedAccess
	};
	
	/**
	  * Converts a path with the given method. Initially used for converting from normal paths to platform extensions (programs make it confusing, so haveing
	  * one source of truth is a good idea)
	 */
	static CORE_API FString ConvertPath(const FString& Path, EPathConversion Method, const TCHAR* ExtraData=nullptr, const TCHAR* OverrideProjectDir=nullptr);
	
	/**
	 * Returns platform and restricted extensions that are present and if bCheckValid is set, valid
     * (for platforms, it uses FDataDrivenPlatformInfo to determine valid platforms, it doesn't just use what's present)
	 *
	 * @return BaseDir and usable extension directories under BaseDir (either Engine or Project)
	 */
	static CORE_API TArray<FString> GetExtensionDirs(const FString& BaseDir, const FString& SubDir, bool bCheckValid);

	enum class EGetExtensionDirsFlags : uint8
	{
		None						= 0,
		// Include the Base directory
		WithBase					= (1 << 0),
		// Include Restricted extension directories
		WithRestricted				= (1 << 1),
		// Include Platform extension directories
		WithPlatforms				= (1 << 2),
		// Use FDataDrivePlatformInfo to determine valid platforms, otherwise just use what's present
		CheckPlatformsValid			= (1 << 3),
		Default						= 0xFF
	};
	FRIEND_ENUM_CLASS_FLAGS(EGetExtensionDirsFlags);

	/**
	 * Returns platform and restricted extensions that are present and if CheckPlatformsValid is set, valid
	 * (for platforms, it uses FDataDrivenPlatformInfo to determine valid platforms, it doesn't just use what's present)
	 *
	 * @return BaseDir and usable extension directories under BaseDir (either Engine or Project)
	 */
	static CORE_API TArray<FString> GetExtensionDirs(
		const FString& BaseDir, 
		const FString& SubDir = FString(), 
		EGetExtensionDirsFlags Flags = EGetExtensionDirsFlags::Default);

	/**
	 * Returns the root directory of the engine directory tree
	 *
	 * @return Root directory.
	 */
	static CORE_API FString RootDir();

#if WITH_EDITOR
	/** Returns the special path used when mounting FeaturePaks in editor */
	static CORE_API const TCHAR* GameFeatureRootPrefix();
#endif

	/**
	 * Returns the base directory of the current project by looking at FApp::GetProjectName().
	 * This is usually a subdirectory of the installation
	 * root directory and can be overridden on the command line to allow self
	 * contained mod support.
	 *
	 * @return base directory
	 */
	static CORE_API FString ProjectDir();

	/**
	* Returns the root directory for user-specific game files.
	*
	* @return game user directory
	*/
	static CORE_API FString ProjectUserDir();

	/**
	 * Returns the content directory of the current game by looking at FApp::GetProjectName().
	 *
	 * @return content directory
	 */
	static CORE_API FString ProjectContentDir();

	/**
	* Returns the directory the root configuration files are located.
	*
	* @return root config directory
	*/
	static CORE_API FString ProjectConfigDir();

	/**
	 * Returns the saved directory of the current game by looking at FApp::GetProjectName().
	 *
	 * @return saved directory
	 */
	static CORE_API const FString& ProjectSavedDir();

	/**
	 * Returns the intermediate directory of the current game by looking at FApp::GetProjectName().
	 *
	 * @return intermediate directory
	 */
	static CORE_API FString ProjectIntermediateDir();

	static CORE_API FString ShaderWorkingDir();

	/**
	 * Returns the plugins directory of the current game by looking at FApp::GetProjectName().
	 *
	 * @return plugins directory
	 */
	static CORE_API FString ProjectPluginsDir();

	/**
	 * Returns the mods directory of the current project by looking at FApp::GetProjectName().
	 *
	 * @return mods directory
	 */
	static CORE_API FString ProjectModsDir();

	/*
	* Returns true if a writable directory for downloaded data that persists across play sessions is available
	*/
	static CORE_API bool HasProjectPersistentDownloadDir();

	/*
	* Returns the writable directory for downloaded data that persists across play sessions.
	*/
	static CORE_API FString ProjectPersistentDownloadDir();

	/**
	 * Returns the directory the engine uses to look for the source leaf ini files. This
	 * can't be an .ini variable for obvious reasons.
	 *
	 * @return source config directory
	 */
	static CORE_API FString SourceConfigDir();

	/**
	 * Returns the directory the engine saves generated config files.
	 *
	 * @return config directory
	 */
	static CORE_API FString GeneratedConfigDir();

	/**
	 * Returns the directory the engine stores sandbox output
	 *
	 * @return sandbox directory
	 */
	static CORE_API FString SandboxesDir();

	/**
	 * Returns the directory the engine uses to output profiling files.
	 *
	 * @return log directory
	 */
	static CORE_API FString ProfilingDir();

	/**
	 * Returns the directory the engine uses to output screenshot files.
	 *
	 * @return screenshot directory
	 */
	static CORE_API FString ScreenShotDir();

	/**
	 * Returns the directory the engine uses to output BugIt files.
	 *
	 * @return screenshot directory
	 */
	static CORE_API FString BugItDir();

	/**
	 * Returns the directory the engine uses to output user requested video capture files.
	 *
	 * @return Video capture directory
	 */
	static CORE_API FString VideoCaptureDir();

	/**
	 * Returns the directory the engine uses to output user requested audio capture files.
	 *
	 * @return Audio capture directory
	 */
	static CORE_API FString AudioCaptureDir();
	
	/**
	 * Returns the directory the engine uses to output logs. This currently can't 
	 * be an .ini setting as the game starts logging before it can read from .ini
	 * files.
	 *
	 * @return log directory
	 */
	static CORE_API FString ProjectLogDir();

	/** Returns the directory for automation save files */
	static CORE_API FString AutomationDir();

	/** Returns the directory for automation save files that are meant to be deleted every run */
	static CORE_API FString AutomationTransientDir();

	/** Returns the directory for results of automation tests. May be deleted every run. */
	static CORE_API FString AutomationReportsDir();

	/** Returns the directory for automation log files */
	static CORE_API FString AutomationLogDir();

	/** Returns the directory for local files used in cloud emulation or support */
	static CORE_API FString CloudDir();

	/**
	 * Returns the directory that contains subfolders for developer-specific content
	 * Example: "../../../ProjectName/Content/Developers
	 */
	static CORE_API FString GameDevelopersDir();

	/**
	 * Returns the name of the subfolder for developer-specific content
	 * Example: "Developers"
	 */
	static CORE_API FStringView DevelopersFolderName();

	/** Returns The folder name for the developer-specific directory for the current user */
	static CORE_API FString GameUserDeveloperFolderName();

	/** Returns The directory that contains developer-specific content for the current user */
	static CORE_API FString GameUserDeveloperDir();

	/** Returns the directory for temp files used for diffing */
	static CORE_API FString DiffDir();

	/** 
	 * Returns a list of engine-specific localization paths
	 */
	static CORE_API const TArray<FString>& GetEngineLocalizationPaths();

	/** 
	 * Returns a list of editor-specific localization paths
	 */
	static CORE_API const TArray<FString>& GetEditorLocalizationPaths();

	/**
	 * Returns a list of property name localization paths
	 */
	static CORE_API const TArray<FString>& GetPropertyNameLocalizationPaths();

	/** 
	 * Returns a list of tool tip localization paths
	 */
	static CORE_API const TArray<FString>& GetToolTipLocalizationPaths();

	/** 
	 * Returns a list of game-specific localization paths
	 */
	static CORE_API const TArray<FString>& GetGameLocalizationPaths();

	/**
	 * Get the name of the platform-specific localization sub-folder
	 */
	static CORE_API FString GetPlatformLocalizationFolderName();

	/** 
	 * Returns a list of restricted/internal folder names (without any slashes) which may be tested against full paths to determine if a path is restricted or not.
	 */
	static CORE_API const TArray<FString>& GetRestrictedFolderNames();

	/** 
	 * Determines if supplied path uses a restricted/internal subdirectory.	Note that slashes are normalized and character case is ignored for the comparison.
	 */
	static CORE_API bool IsRestrictedPath(const FString& InPath);

	/**
	 * Returns the saved directory that is not game specific. This is usually the same as
	 * EngineSavedDir().
	 *
	 * @return saved directory
	 */
	static CORE_API FString GameAgnosticSavedDir();

	/** Returns the directory where engine source code files are kept */
	static CORE_API FString EngineSourceDir();

	/** Returns the directory where game source code files are kept */
	static CORE_API FString GameSourceDir();

	/** Returns the directory where feature packs are kept */
	static CORE_API FString FeaturePackDir();

	/**
	 * Checks whether the path to the project file, if any, is set.
	 *
	 * @return true if the path is set, false otherwise.
	 */
	static CORE_API bool IsProjectFilePathSet();
	
	/**
	 * Gets the path to the project file.
	 *
	 * @return Project file path.
	 */
	static CORE_API FString GetProjectFilePath();

	/**
	 * Sets the path to the project file.
	 *
	 * @param NewGameProjectFilePath - The project file path to set.
	 */
	static CORE_API void SetProjectFilePath( const FString& NewGameProjectFilePath );

	/**
	 * Gets the extension for this filename.
	 * 
	 * Default: Dot is not included, compound extensions are not allowed.
	 * @param Flags
	 *		EFlags::IncludeDot Includes the leading dot in the result.
	 *		EFlags::AllowCompoundExtension Allows compound extensions (.a.b) in the result.
	 *
	 * @return	the extension of this filename, or an empty string if the filename doesn't have an extension.
	 */
	static CORE_API FString GetExtension(const FString& InPath, UE::Paths::EFlags Flags = UE::Paths::EFlags::None);
	/** Legacy version of GetExtension that uses a bool to specify bIncludeDot. EFlags::AllowCompoundExtension == 0. */
	static CORE_API FString GetExtension(const FString& InPath, bool bIncludeDot);

	// Returns the filename (with extension), minus any path information.
	static CORE_API FString GetCleanFilename(const FString& InPath);
	static CORE_API FString GetCleanFilenameUtf8(const FUtf8String& InPath);

	// Returns the filename (with extension), minus any path information.
	static CORE_API FString GetCleanFilename(FString&& InPath);

	// Returns the same thing as GetCleanFilename, but without the extension
	static CORE_API FString GetBaseFilename(const FString& InPath, bool bRemovePath=true );

	// Returns the same thing as GetCleanFilename, but without the extension
	static CORE_API FString GetBaseFilename(FString&& InPath, bool bRemovePath = true);

	// Returns the path in front of the filename
	static CORE_API FString GetPath(const FString& InPath);

	// Returns the path in front of the filename
	static CORE_API FString GetPath(FString&& InPath);

	// Returns the leaf in the path
	static CORE_API FString GetPathLeaf(const FString& InPath);

	// Returns the leaf in the path
	static CORE_API FString GetPathLeaf(FString&& InPath);

	/** Changes the extension of the given filename (does nothing if the file has no extension) */
	static CORE_API FString ChangeExtension(const FString& InPath, const FString& InNewExtension);

	/** Sets the extension of the given filename (like ChangeExtension, but also applies the extension if the file doesn't have one) */
	static CORE_API FString SetExtension(const FString& InPath, const FString& InNewExtension);

	/** Returns true if this file was found, false otherwise */
	static CORE_API bool FileExists(const FString& InPath);

	/** Returns true if this directory was found, false otherwise */
	static CORE_API bool DirectoryExists(const FString& InPath);

	/** Returns true if this path represents a root drive or volume */
	static CORE_API bool IsDrive(const FString& InPath);

	/** Returns true if this path is relative to another path */
	static CORE_API bool IsRelative(const FString& InPath);

	/** Convert all / and \ to TEXT("/") */
	static CORE_API void NormalizeFilename(FString& InPath);

	/** Converts the path casing to match the casing found on disk without resolving directory junctions */
	static CORE_API FString FindCorrectCase(const FString& Path);

	/**
	 * Checks if two paths are the same.
	 *
	 * @param PathA First path to check.
	 * @param PathB Second path to check.
	 *
	 * @returns True if both paths are the same. False otherwise.
	 */
	static CORE_API bool IsSamePath(const FString& PathA, const FString& PathB);

	/** Determines if a path is under a given directory */
	static CORE_API bool IsUnderDirectory(const FString& InPath, const FString& InDirectory);

	/** Normalize all / and \ to TEXT("/") and remove any trailing TEXT("/") if the character before that is not a TEXT("/") or a colon */
	static CORE_API void NormalizeDirectoryName(FString& InPath);

	/**
	 * Takes a fully pathed string and eliminates relative pathing (eg: annihilates ".." with the adjacent directory).
	 * Assumes all slashes have been converted to TEXT('/').
	 * For example, takes the string:
	 *	BaseDirectory/SomeDirectory/../SomeOtherDirectory/Filename.ext
	 * and converts it to:
	 *	BaseDirectory/SomeOtherDirectory/Filename.ext
	 * @param bCollapseAllPossible If false, and there are leading .. that are uncollapsible, the function
	 *        early exits and does not modify .. or . that occur later in the path. If true, the function
	 *        collapses all . and .. that can be collapsed. Return value is the same in either case.
	 * @return true if all directories were collapsed, else false.
	 */
	static CORE_API bool CollapseRelativeDirectories(FString& InPath, bool bCollapseAllPossible = false);

	/**
	 * Removes duplicate slashes in paths.
	 * Assumes all slashes have been converted to TEXT('/').
	 * For example, takes the string:
	 *	BaseDirectory/SomeDirectory//SomeOtherDirectory////Filename.ext
	 * and converts it to:
	 *	BaseDirectory/SomeDirectory/SomeOtherDirectory/Filename.ext
	 */
	static CORE_API void RemoveDuplicateSlashes(FString& InPath);

	 /** Returns a copy of the given path on which duplicate slashes were removed (see the inplace version for more details). */
	static CORE_API FString RemoveDuplicateSlashes(const FString& InPath);

	/**
	 * Make fully standard "Unreal" pathname:
	 *    - Normalizes path separators [NormalizeFilename]
	 *    - Removes extraneous separators  [NormalizeDirectoryName, as well removing adjacent separators]
	 *    - Collapses internal ..'s
	 *    - Makes relative to Engine\Binaries\<Platform> (will ALWAYS start with ..\..\..)
	 */
	static CORE_API FString CreateStandardFilename(const FString& InPath);

	static CORE_API void MakeStandardFilename(FString& InPath);

	/** Takes an "Unreal" pathname and converts it to a platform filename. */
	static CORE_API void MakePlatformFilename(FString& InPath);

	/** 
	 * Assuming both paths (or filenames) are relative to the same base dir, modifies InPath to be relative to InRelativeTo
	 *
	 * @param InPath Path to change to be relative to InRelativeTo
	 * @param InRelativeTo Path to use as the new relative base
	 * @returns true if InPath was changed to be relative
	 */
	static CORE_API bool MakePathRelativeTo( FString& InPath, const TCHAR* InRelativeTo );

	/**
	 * Converts a relative path name to a fully qualified name relative to the process BaseDir().
	 */
	static CORE_API FString ConvertRelativePathToFull(const FString& InPath);

	/**
	 * Converts a relative path name to a fully qualified name relative to the process BaseDir().
	 */
	static CORE_API FString ConvertRelativePathToFull(FString&& InPath);

	/**
	 * Converts a relative path name to a fully qualified name relative to the specified BasePath.
	 */
	static CORE_API FString ConvertRelativePathToFull(const FString& BasePath, const FString& InPath);

	/**
	 * Converts a relative path name to a fully qualified name relative to the specified BasePath.
	 */
	static CORE_API FString ConvertRelativePathToFull(const FString& BasePath, FString&& InPath);

	/**
	 * Converts a relative path name to a fully qualified name relative to the specified BasePath.
	 */
	static CORE_API FString ConvertRelativePathToFull(FString&& BasePath, const FString& InPath);

	/**
	 * Converts a relative path name to a fully qualified name relative to the specified BasePath.
	 */
	static CORE_API FString ConvertRelativePathToFull(FString&& BasePath, FString&& InPath);

	/**
	 * Converts a normal path to a sandbox path (in Saved/Sandboxes).
	 *
	 * @param InSandboxName The name of the sandbox.
	 */
	static CORE_API FString ConvertToSandboxPath( const FString& InPath, const TCHAR* InSandboxName );

	/**
	 * Converts a sandbox (in Saved/Sandboxes) path to a normal path.
	 *
	 * @param InSandboxName The name of the sandbox.
	 */
	static CORE_API FString ConvertFromSandboxPath( const FString& InPath, const TCHAR* InSandboxName );

	/** 
	 * Creates a temporary filename with the specified prefix.
	 *
	 * @param Path The file pathname.
	 * @param Prefix The file prefix.
	 * @param Extension File extension ('.' required).
	 */
	static CORE_API FString CreateTempFilename( const TCHAR* Path, const TCHAR* Prefix = TEXT(""), const TCHAR* Extension = TEXT(".tmp") );

	/**
	* Returns a string containing all invalid characters as dictated by the operating system
	*/
	static CORE_API FString GetInvalidFileSystemChars();

	/**
	*	Returns a string that is safe to use as a filename because all items in
	*	GetInvalidFileSystemChars() are removed
	*/
	static CORE_API FString MakeValidFileName(const FString& InString, const TCHAR InReplacementChar = TEXT('\0'));

	/** 
	 * Validates that the parts that make up the path contain no invalid characters as dictated by the operating system
	 * Note that this is a different set of restrictions to those imposed by FPackageName
	 *
	 * @param InPath - path to validate
	 * @param OutReason - optional parameter to fill with the failure reason
	 */
	static CORE_API bool ValidatePath( const FString& InPath, FText* OutReason = nullptr );

	/**
	 * Parses a fully qualified or relative filename into its components (filename, path, extension).
	 *
	 * @param	InPath			[in] Full filename path
	 * @param	PathPart		[out] receives the value of the path portion of the input string
	 * @param	FilenamePart	[out] receives the value of the filename portion of the input string
	 * @param	ExtensionPart	[out] receives the value of the extension portion of the input string
	 */
	static CORE_API void Split( const FString& InPath, FString& PathPart, FString& FilenamePart, FString& ExtensionPart );

	/** Gets the relative path to get from BaseDir to RootDirectory  */
	static CORE_API const FString& GetRelativePathToRoot();

	template <typename... PathTypes>
	UE_FORCEINLINE_HINT static FString Combine(PathTypes&&... InPaths)
	{
		return CombineImpl<PathTypes...>(Forward<PathTypes>(InPaths)...);
	}

	/**
	 * Frees any memory retained by FPaths.
	 */
	static CORE_API void TearDown();

protected:

	static CORE_API FString CombineInternal(const FStringView* Paths, int32 NumPaths);
	/**
	 * Helper function to reduce duplication between NormalizeFilename and other functions that need to normalize
	 * while also doing other transformations. Does not do all steps of NormalizeFilename, see the comments in the
	 * function for details of which steps it handles.
	 */
	static void PlatformIndependentNormalizePath(FString& Path, bool bRemoveDuplicateSlashes,
		int32& OutFirstColonIndex, int32& OutFirstSlashIndex);

private:
	struct FStaticData;

	template <typename... PathTypes>
	inline static FString CombineImpl(UE::Core::Private::TToStringType_T<std::decay_t<PathTypes>>... InPaths)
	{
		const FStringView Paths[] = { FStringView(InPaths)... };
		return CombineInternal(Paths, UE_ARRAY_COUNT(Paths));
	}

	/** Returns, if any, the value of the -userdir command line argument. This can be used to sandbox artifacts to a desired location */
	static CORE_API const FString& CustomUserDirArgument();

	/** Returns, if any, the value of the -shaderworkingdir command line argument. This can be used to sandbox shader working files to a desired location */
	static CORE_API const FString& CustomShaderDirArgument();
};

ENUM_CLASS_FLAGS(FPaths::EGetExtensionDirsFlags);
