// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/DelayedAutoRegister.h"

class FEvent;

struct FPreLoadFile : public FDelayedAutoRegisterHelper
{
	CORE_API FPreLoadFile(const TCHAR* InPath);

	CORE_API void* TakeOwnershipOfLoadedData(int64* OutFileSize=nullptr);
	static CORE_API void* TakeOwnershipOfLoadedDataByPath(const TCHAR* Filename, int64* OutFileSize);

protected:
	CORE_API void KickOffRead();

	bool bIsComplete;
	bool bFailedToOpenInKickOff;
	static CORE_API bool bSystemNoLongerTakingRequests;
	void* Data;
	int64 FileSize;
	FString Path;
	FEvent* CompletionEvent;
	class IAsyncReadFileHandle* AsyncReadHandle = nullptr;
	class IAsyncReadRequest* SizeRequestHandle = nullptr;
};
