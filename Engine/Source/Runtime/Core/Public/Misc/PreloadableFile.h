// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Templates/Atomic.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class FEvent;
class IAsyncReadFileHandle;
class IAsyncReadRequest;
class IFileHandle;

#define FPRELOADABLEFILE_TEST_ENABLED 0

/**
 * A read-only archive that adds support for asynchronous preloading and priming to an inner archive.
 *
 * This class supports two mutually-exclusive modes:
 *   PreloadBytes:
 *     An asynchronous inner archive is opened using the passed-in CreateAsyncArchive function; this call is made asynchronously on a TaskGraph thread.
 *     The size is read during initialization.
 *     After initialization, when StartPreload is called, an array of bytes equal in size to the inner archive's size is allocated,
 *       and an asynchronous ReadRequest is sent to the IAsyncReadFileHandle to read the first <PageSize> bytes of the file.
 *     Upon completion of each in-flight ReadRequest, another asynchronous ReadRequest is issued, until the entire file has been read.
 *     If serialize functions are called beyond the bytes of the file that have been cached so far, they requests are satisfied by synchronous reads.
 *
 *     Activate this mode by passing an FCreateArchive to InitializeAsync.
 *  PreloadHandle:
 *     A synchronous inner archive is opened using the passed-in CreateArchive function; this call is made asynchronously on a TaskGraph thread.
 *     Optionally, a precache request is sent to the inner archive for the first <PrimeSize> bytes; this call is also made asynchronously.
 *     The created and primed lower-level FArchive can then be detached from this class and handed off to a new owner.
 *
 *    Activate this mode by passing an FCreateAsyncArchive and (optionally, for the precache request) Flags::Prime to InitializeAsync.
 *
 * This class is not threadsafe. The public interface can be used at the same time as internal asynchronous tasks are executing, but the
 * public interface can not be used from multiple threads at once.
 */
class FPreloadableArchive : public FArchive
{
public:
	typedef TUniqueFunction<FArchive* ()> FCreateArchive;
	typedef TUniqueFunction<IAsyncReadFileHandle* ()> FCreateAsyncArchive;

	enum Flags
	{
		None = 0x0,

		// Mode (mutually exclusive)
		ModeBits = 0x1,
		PreloadHandle = 0x0,	// Asynchronously open the Lower-Level Archive, but do not read bytes from it. The Lower-Level Archive can be detached or can be accessed through Serialize.
		PreloadBytes = 0x1,		// Asynchronously open the Lower-Level Archive and read bytes from it into an In-Memory cache of the file. Serialize will read from the cached bytes if available, otherwise will read from the Lower-Level Archive.

		// Options (independently selectable, do not necessarily apply to all modes)
		Prime = 0x2,			// Only applicable to PreloadHandle mode. After asynchronously opening the LowerLevel archive, asychronously call Precache(0, <PrimeSize>).
	};
	enum
	{
		DefaultPrimeSize = 1024,	// The default argument for PrimeSize in IntializeAsync. How many bytes are requested in the initial PrimeSize call.
		DefaultPageSize = 64*1024	// The default size of read requests made to the LowerLevel Archive in PreloadBytes mode when reading bytes into the In-Memory cache.
	};

	CORE_API FPreloadableArchive(FStringView ArchiveName);
	CORE_API virtual ~FPreloadableArchive();

	// Initialization
	/** Set the PageSize used read requests made to the LowerLevel Archive in PreloadBytes mode when reading bytes into the In-Memory cache. Invalid to set after Initialization; PageSize must be constant during use. */
	CORE_API void SetPageSize(int64 PageSize);
	/** Initialize the FPreloadableFile asynchronously, performing FileOpen operations on another thread. Use IsInitialized or WaitForInitialization to check progress. */
	CORE_API void InitializeAsync(FCreateArchive&& InCreateArchiveFunction, uint32 InFlags = Flags::None, int64 PrimeSize = DefaultPrimeSize);
	CORE_API void InitializeAsync(FCreateAsyncArchive&& InCreateAsyncArchiveFunction, uint32 InFlags = Flags::None, int64 PrimeSize = DefaultPrimeSize);
	/** Return whether InitializeAsync has completed. If Close is called, state returns to false until the next call to InitializeAsync. */
	CORE_API bool IsInitialized() const;
	/** Wait for InitializeAsync to complete if it is running, otherwise return immediately. */
	CORE_API void WaitForInitialization() const;

	// Preloading
	/** When in PreloadBytes mode, if not already preloading, allocate if necessary the memory for the preloaded bytes and start the chain of asynchronous ReadRequests for the bytes. Returns whether preloading is now active. */
	CORE_API bool StartPreload();
	/** Cancel any current asynchronous ReadRequests and wait for the asynchronous work to exit. */
	CORE_API void StopPreload();
	/** Return whether preloading is in progress. Value may not be up to date since asynchronous work might complete in a race condition. */
	CORE_API bool IsPreloading() const;
	/** When in PreloadBytes mode, allocate if necessary the memory for the preloaded bytes. Return whether the memory is now allocated. */
	CORE_API bool AllocateCache();
	/** Free all memory used by the cache or for preloading (calling StopPreload if necessary). */
	CORE_API void ReleaseCache();
	/** Return whether the cache is currently allocated. */
	CORE_API bool IsCacheAllocated() const;
	/** Return the LowerLevel FArchive if it has been allocated. May return null, even if the FPreloadableFile is currently active. If return value is non-null, caller is responsible for deleting it. */
	CORE_API FArchive* DetachLowerLevel();
	/** Return whether this archive has an inner archive it can provide data out of. Serialize will fail if this returns false. */
	CORE_API bool HasValidData() const;

	// FArchive
	CORE_API virtual void Serialize(void* V, int64 Length) final;
	CORE_API virtual void Seek(int64 InPos) final;
	CORE_API virtual int64 Tell() final;
	/** Return the size of the file, or -1 if the file does not exist. This is also the amount of memory that will be allocated by AllocateCache. */
	CORE_API virtual int64 TotalSize() final;
	CORE_API virtual bool Close() final;
	CORE_API virtual FString GetArchiveName() const final;

protected:
	/** Helper function for InitializeAsync, sets up the asynchronous call to InitializeInternal */
	CORE_API void InitializeInternalAsync(FCreateArchive&& InCreateArchiveFunction, FCreateAsyncArchive&& InCreateAsyncArchiveFunction, uint32 InFlags, int64 PrimeSize);
	/** Helper function for InitializeAsync, called from a TaskGraph thread. */
	CORE_API void InitializeInternal(FCreateArchive&& InCreateArchiveFunction, FCreateAsyncArchive&& InCreateAsyncArchiveFunction, uint32 Flags, int64 PrimeSize);
#if FPRELOADABLEFILE_TEST_ENABLED
	CORE_API void SerializeInternal(void* V, int64 Length);
#endif
	CORE_API void PausePreload();
	CORE_API void ResumePreload();
	CORE_API bool ResumePreloadNonRecursive();
	CORE_API void OnReadComplete(bool bCanceled, IAsyncReadRequest* ReadRequest);
	CORE_API void FreeRetiredRequests();
	CORE_API void SerializeSynchronously(void* V, int64 Length);

	FString ArchiveName;
	/** The Offset into the file or preloaded bytes that will be used in the next call to Serialize. */
	int64 Pos = 0;
	/** The number of bytes in the file. */
	int64 Size = -1;

	/** An Event used for synchronization with asynchronous tasks - InitializingAsync or receiving ReadRequests from the AsynchronousHandle. */
	FEvent* PendingAsyncComplete = nullptr;
	/** Threadsafe variable that returns true only after all asynchronous initialization is complete. Is also reset to false when public-interface users call Close(). */
	TAtomic<bool> bInitialized;
	/** Threadsafe variable that is true only during the period between initialization until Preloading stops (either due to EOF reached or due to Serialize turning it off. */
	TAtomic<bool> bIsPreloading;
	/** Variable that is set to true from the public interface thread to signal that (temporarily) no further ReadRequests should be sent when the currently active one completes. */
	TAtomic<bool> bIsPreloadingPaused;

	/** An array of bytes of size Size. Non-null only in PreloadBytes mode and in-between calls to AllocateCache/ReleaseCache. */
	uint8* CacheBytes = nullptr;
	/**
	 * Number of bytes in CacheBytes that have already been read. This is used in Serialize to check which bytes are available and in preloading to know for which bytes to issue a read request.
	 * This variable is read-only threadsafe. It is guaranteed to be written only after the bytes in CacheBytes have finished writing, and it is guaranteed to be written before bIsPreloading is written.
	 * It is not fully threadsafe; threads that need to write to CacheEnd need to do their Read/Write within the PreloadLock CriticalSection.
	 */
	TAtomic<int64> CacheEnd;

	/** The handle used for PreloadBytes mode, to fulfull ReadReqeusts. */
	TUniquePtr<IAsyncReadFileHandle> AsynchronousHandle;
	/** The archive used in PreloadHandle mode or to service Serialize requests that are beyond CacheEnd when in PreloadBytes mode. */
	TUniquePtr<FArchive> SynchronousArchive;
#if FPRELOADABLEFILE_TEST_ENABLED
	/** A duplicate handle used in serialize to validate the returned bytes are correct. */
	TUniquePtr<FArchive> TestArchive;
#endif
	/** ReadRequests that have completed but we have not yet deleted. */
	TArray<IAsyncReadRequest*> RetiredRequests;
	/** The number of bytes requested from the AsynchronousHandle in each ReadRequest. Larger values have slightly faster results due to per-call overhead, but add greater latency to Serialize calls that read past the end of the cache. */
	int64 PageSize = DefaultPageSize;

	/** CriticalSection used to synchronize access to the CacheBytes. */
	FCriticalSection PreloadLock;
	/** Set to true if OnReadComplete is called inline on the same thread from ReadRequest; we need special handling for this case. */
	bool bReadCompleteWasCalledInline = false;
	/** Set to true during the ReadRequest call to allow us to detect if OnReadComplete is called inline on the same thread from ReadRequest. */
	bool bIsInlineReadComplete = false;

	/** Saved values from the inline OnReadComplete call */
	struct FSavedReadCompleteArguments
	{
	public:
		void Set(bool bInCanceled, IAsyncReadRequest* InReadRequest)
		{
			bCanceled = bInCanceled;
			ReadRequest = InReadRequest;
		}
		void Get(bool& bOutCanceled, IAsyncReadRequest*& OutReadRequest)
		{
			bOutCanceled = bCanceled;
			OutReadRequest = ReadRequest;
			ReadRequest = nullptr;
		}
	private:
		bool bCanceled = false;
		IAsyncReadRequest* ReadRequest = nullptr;
	}
	SavedReadCompleteArguments;

	friend class FPreloadableFileProxy;
};

/**
 * An FPreloadableArchive that is customized for reading files from IFileManager.
 *
 * This class also supports registration of instances of this class by filename, which other systems in the engine can use to request
 *   an FArchive for the preload file, if it exists, replacing a call they would otherwise make to IFileManager::Get().CreateFileReader.
 *
 * As with the base class, the preloading can work in either PreloadBytes or PreloadHandle mode.
 *
 * Activate PreloadBytes mode by passing Flags::PreloadBytes to InitializeAsync.
 * Activate PreloadHandle mode by passing Flags::PreloadHandle to InitializeAsync, optionally or'd with Flags::Prime.
 *
 */
class FPreloadableFile : public FPreloadableArchive
{
public:
	CORE_API FPreloadableFile(FStringView FileName);

	/** Initialize the FPreloadableFile asynchronously, performing FileOpen operations on another thread. Use IsInitialized or WaitForInitialization to check progress. */
	CORE_API void InitializeAsync(uint32 InFlags = Flags::None, int64 PrimeSize=DefaultPrimeSize);

	// Registration
	/**
	 * Try to register the given FPreloadableFile instance to handle the next call to TryTakeArchive for its FileName.
	 * Will fail if the instance has not been initialized or if another instance has already registered for the Filename.
	 * Return whether the instance is currently registered. Returns true if the instance was already registered.
	 * Registered files are referenced-counted, and the reference will not be dropped until (1) (TryTakeArchive or UnRegister is called) and (2) (PreloadBytes mode only) the archive returned from TryTakeArchive is deleted.
	 */
	static CORE_API bool TryRegister(const TSharedPtr<FPreloadableFile>& PreloadableFile);
	/**
	 * Look up an FPreloadableFile instance registered for the given FileName, and return an FArchive from it.
	 * If found, removes the registration so no future call to TryTakeArchive can sue the same FArchive.
	 * If the instance is in PreloadHandle mode, the Lower-Level FArchive will be detached from the FPreloadableFile and returned using DetachLowerLevel.
	 * If the instance is in PreloadBytes mode, a ProxyArchive will be returned that forwards call to the FPreloadableFile instance.
	 */
	static CORE_API FArchive* TryTakeArchive(const TCHAR* FileName);
	/** Remove the FPreloadableFile instance if it is registered for its FileName. Returns whether the instance was registered. */
	static CORE_API bool UnRegister(const TSharedPtr<FPreloadableFile>& PreloadableFile);

private:
	/** Map used for TryTakeArchive registration. */
	static TMap<FString, TSharedPtr<FPreloadableFile>> RegisteredFiles;
};

/**
 * A helper class for systems that want to make their own registration system.
 * A proxy archive that keeps a shared pointer to the inner FPreloadableArchive, so that the inner preloadablearchive will remain
 * alive until at least as long as the proxy is destroyed.
 */
class FPreloadableArchiveProxy : public FArchive
{
public:
	explicit FPreloadableArchiveProxy(const TSharedPtr<FPreloadableArchive>& InArchive)
		:Archive(InArchive)
	{
		check(Archive);
	}
	virtual void Seek(int64 InPos) final
	{
		Archive->Seek(InPos);
	}
	virtual int64 Tell() final
	{
		return Archive->Tell();
	}
	virtual int64 TotalSize() final
	{
		return Archive->TotalSize();
	}
	virtual bool Close() final
	{
		return Archive->Close();
	}
	virtual void Serialize(void* V, int64 Length) final
	{
		Archive->Serialize(V, Length);
	}
	virtual FString GetArchiveName() const final
	{
		return Archive->GetArchiveName();
	}

private:
	TSharedPtr<FPreloadableArchive> Archive;
};

