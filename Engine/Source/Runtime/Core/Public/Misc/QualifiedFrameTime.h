// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/Timecode.h"

/**
 * A frame time qualified by a frame rate context
 */
struct FQualifiedFrameTime
{

	/**
	 * Default construction for UObject purposes
	 */
	FQualifiedFrameTime()
		: Time(0), Rate(24, 1)
	{}

	/**
	 * User construction from a frame time and its frame rate
	 */
	FQualifiedFrameTime(const FFrameTime& InTime, const FFrameRate& InRate)
		: Time(InTime), Rate(InRate)
	{}

	/**
	 * User construction from a timecode and its frame rate
	 */
	FQualifiedFrameTime(const FTimecode& InTimecode, const FFrameRate& InRate)
		: Time(InTimecode.ToFrameNumber(InRate), InTimecode.Subframe)
		, Rate(InRate)
	{
	}

public:

	/**
	 * Convert this frame time to a value in seconds
	 */
	double AsSeconds() const
	{
		return Time / Rate;
	}

	/**
	 * Convert this frame time to a different frame rate
	 */
	FFrameTime ConvertTo(FFrameRate DesiredRate) const
	{
		return  FFrameRate::TransformTime(Time, Rate, DesiredRate);
	}

	/**
	 * Create an FTimecode from this qualified frame time.
	 *
	 * Whether or not the returned timecode is a drop-frame timecode will be determined by the qualified frame time's frame rate
	 * and the CVar specifying whether to generate drop-frame timecodes by default for supported frame rates.
	 */
	FTimecode ToTimecode() const
	{
		return FTimecode::FromFrameTime(Time, Rate);
	}

	/**
	 * Create an FTimecode from this qualified frame time. Optionally supports creating a drop-frame timecode,
	 * which drops certain timecode display numbers to help account for NTSC frame rates which are fractional.
	 *
	 * @param bDropFrame    If true, the returned timecode will drop the first two frames on every minute (except when Minute % 10 == 0).
	 *     This is only valid for NTSC framerates (29.97, 59.94) and will assert if you try to create a drop-frame timecode
	 *     from a non-valid framerate. All framerates can be represented by non-drop timecode.
	 */
	FTimecode ToTimecode(bool bDropFrame) const
	{
		return FTimecode::FromFrameTime(Time, Rate, bDropFrame);
	}

public:

	/** IMPORTANT: If you change the struct data, ensure that you also update the version in NoExportTypes.h  */

	/** The frame time */
	FFrameTime Time;

	/** The rate that this frame time is in */
	FFrameRate Rate;
};
