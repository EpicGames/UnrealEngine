// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "GenericPlatform/GenericPlatformAffinity.h"
#include "Templates/Function.h"

class IQueuedWork;

/** Higher priority are picked up first by the task thread pool. */
enum class EQueuedWorkPriority : uint8
{
	Blocking = 0,
	Highest = 1,
	High = 2,
	Normal = 3,
	Low = 4,
	Lowest = 5,
	Count
};

inline const TCHAR* LexToString(EQueuedWorkPriority Priority)
{
	switch (Priority)
	{
	case EQueuedWorkPriority::Blocking:
		return TEXT("Blocking");
	case EQueuedWorkPriority::Highest:
		return TEXT("Highest");
	case EQueuedWorkPriority::High:
		return TEXT("High");
	case EQueuedWorkPriority::Normal:
		return TEXT("Normal");
	case EQueuedWorkPriority::Low:
		return TEXT("Low");
	case EQueuedWorkPriority::Lowest:
		return TEXT("Lowest");
	case EQueuedWorkPriority::Count:
	default:
		check(false);
		return TEXT("Unknown");
	}
}

/** 
 *  Priority Queue tailored for FQueuedThreadPool implementation
 *
 *  This class is NOT thread-safe and must be properly protected.
 */
class FThreadPoolPriorityQueue
{
public:
	CORE_API FThreadPoolPriorityQueue();

	/**
	 * Enqueue a work item at specified priority
	 */
	CORE_API void Enqueue(IQueuedWork* InQueuedWork, EQueuedWorkPriority InPriority = EQueuedWorkPriority::Normal);

	/**
	 * Search and remove a queued work item from the list
	 */
	CORE_API bool Retract(IQueuedWork* InQueuedWork);

	/**
	 * Get the next work item in priority order.
	 */
	CORE_API IQueuedWork* Dequeue(EQueuedWorkPriority* OutDequeuedWorkPriority = nullptr);

	/**
	 * Get the next work item in priority order without actually dequeuing.
	 */
	CORE_API IQueuedWork* Peek(EQueuedWorkPriority* OutDequeuedWorkPriority = nullptr) const;

	/**
	 * Empty the queue.
	 */
	CORE_API void Reset();

	/**
	 * Get the total number of queued items.
	 */
	int32 Num() const { return NumQueuedWork; }

	/**
	 * Sort Priority Bucket given Predicate
	 */
	CORE_API void Sort(EQueuedWorkPriority InPriorityBucket, TFunctionRef<bool (const IQueuedWork* A, const IQueuedWork* B)> Predicate);
private:
	/** The first queue to extract a work item from to avoid scanning all priorities when unqueuing. */
	int32 FirstNonEmptyQueueIndex = 0;
	TArray<TArray<IQueuedWork*>, TInlineAllocator<static_cast<int32>(EQueuedWorkPriority::Lowest) + 1>> PriorityQueuedWork;
	TAtomic<int32> NumQueuedWork;
};

/**
 * Interface for queued thread pools.
 *
 * This interface is used by all queued thread pools. It used as a callback by
 * FQueuedThreads and is used to queue asynchronous work for callers.
 */
class FQueuedThreadPool
{
public:
	/**
	 * Creates the thread pool with the specified number of threads
	 *
	 * @param InNumQueuedThreads Specifies the number of threads to use in the pool
	 * @param StackSize The size of stack the threads in the pool need (32K default)
	 * @param ThreadPriority priority of new pool thread
	 * @param Name optional name for the pool to be used for instrumentation
	 * @return Whether the pool creation was successful or not
	 */
	virtual bool Create(uint32 InNumQueuedThreads, uint32 StackSize = (32 * 1024), EThreadPriority ThreadPriority = TPri_Normal, const TCHAR* Name = TEXT("UnknownThreadPool")) = 0;

	/**
	 * Creates the thread pool with the specified number of forkable threads (see FForkProcessHelper)
	 * 
	 * Currently only works after the process has been forked because GetSingleThreadInterface() isn't implemented for pooled threads
	 *
	 * @param InNumQueuedThreads Specifies the number of threads to use in the pool
	 * @param StackSize The size of stack the threads in the pool need (32K default)
	 * @param ThreadPriority priority of new pool thread
	 * @param Name optional name for the pool to be used for instrumentation
	 * @return Whether the pool creation was successful or not
	 */
	virtual bool CreateForkable(uint32 InNumQueuedThreads, uint32 StackSize = (32 * 1024), EThreadPriority ThreadPriority = TPri_Normal, const TCHAR* Name = TEXT("UnknownThreadPool"))
	{
		return false;
	}

	/** Tells the pool to clean up all background threads */
	virtual void Destroy() = 0;

	/**
	 * Checks to see if there is a thread available to perform the task. If not,
	 * it queues the work for later. Otherwise it is immediately dispatched.
	 *
	 * @param InQueuedWork         The work that needs to be done asynchronously
	 * @param InQueuedWorkPriority The priority at which to process this task
	 * @see RetractQueuedWork
	 */
	virtual void AddQueuedWork( IQueuedWork* InQueuedWork, EQueuedWorkPriority InQueuedWorkPriority = EQueuedWorkPriority::Normal) = 0;

	/**
	 * Attempts to retract a previously queued task.
	 *
	 * @param InQueuedWork The work to try to retract
	 * @return true if the work was retracted
	 * @see AddQueuedWork
	 */
	virtual bool RetractQueuedWork(IQueuedWork* InQueuedWork) = 0;

	/**
	 * Get the number of queued threads
	 */
	virtual int32 GetNumThreads() const = 0;

public:
			CORE_API FQueuedThreadPool();
	CORE_API virtual	~FQueuedThreadPool();

public:

	/**
	 * Allocates a thread pool
	 *
	 * @return A new thread pool.
	 */
	static CORE_API FQueuedThreadPool* Allocate();

	/**
	 *	Stack size for threads created for the thread pool. 
	 *	Can be overridden by other projects.
	 *	If 0 means to use the value passed in the Create method.
	 */
	static CORE_API uint32 OverrideStackSize;
};

/**
 *  Global thread pool for shared async operations
 */
extern CORE_API FQueuedThreadPool* GThreadPool;

extern CORE_API FQueuedThreadPool* GIOThreadPool;

extern CORE_API FQueuedThreadPool* GBackgroundPriorityThreadPool;

#if WITH_EDITOR
extern CORE_API FQueuedThreadPool* GLargeThreadPool;
#endif
