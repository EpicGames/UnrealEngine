// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Fundamental/Scheduler.h"
#include "Async/Fundamental/Task.h"
#include "Async/ManualResetEvent.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "Experimental/Containers/FAAArrayQueue.h"
#include "Experimental/Misc/ExecutionResource.h"
#include "GenericPlatform/GenericPlatformAffinity.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "HAL/PlatformAffinity.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformProcess.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "Misc/IQueuedWork.h"
#include "Misc/MemStack.h"
#include "Misc/ScopeLock.h"
#include "QueuedThreadPool.h"
#include "ScopeRWLock.h"
#include "Stats/Stats.h"
#include "Templates/Atomic.h"
#include "Templates/Function.h"
#include "Templates/RefCounting.h"

#include <atomic>

inline constexpr auto DefaultQueuedWorkPriorityWrapper = [](EQueuedWorkPriority InPriority) { return InPriority; };

/** ThreadPool wrapper implementation allowing to schedule
  * up to MaxConcurrency tasks at a time making sub-partitioning
  * another thread-pool a breeze and allowing more fine-grained control
  * over scheduling by effectively giving another set of priorities.
  */
class FQueuedThreadPoolWrapper : public FQueuedThreadPool
{
public:
	/**
	 * InWrappedQueuedThreadPool  Underlying thread pool to schedule task to.
	 * InMaxConcurrency           Maximum number of concurrent tasks allowed, -1 will limit concurrency to number of threads available in the underlying thread pool.
	 * InPriorityMapper           Thread-safe function used to map any priority from this Queue to the priority that should be used when scheduling the task on the underlying thread pool.
	 */
	CORE_API FQueuedThreadPoolWrapper(FQueuedThreadPool* InWrappedQueuedThreadPool, int32 InMaxConcurrency = -1, TFunction<EQueuedWorkPriority(EQueuedWorkPriority)> InPriorityMapper = DefaultQueuedWorkPriorityWrapper);
	CORE_API ~FQueuedThreadPoolWrapper() override;

	/**
	 *  Queued task are not scheduled against the wrapped thread-pool until resumed
	 */
	CORE_API void Pause();

	/**
	 *  Resume a specified amount of queued work, or -1 to unpause.
	 */
	CORE_API void Resume(int32 InNumQueuedWork = -1);

	/**
	 *  Dynamically adjust the maximum number of concurrent tasks, -1 for unlimited.
	 */
	CORE_API void SetMaxConcurrency(int32 MaxConcurrency = -1);

	/**
	 *  Return the estimated memory currently used by this thread pool, if supported, or -1 if not.
	 */
	virtual int64 GetEstimatedMemory() const
	{
		return -1ll;
	}

	CORE_API void AddQueuedWork(IQueuedWork* InQueuedWork, EQueuedWorkPriority InPriority = EQueuedWorkPriority::Normal) override;
	CORE_API bool RetractQueuedWork(IQueuedWork* InQueuedWork) override;
	CORE_API int32 GetNumThreads() const override;
	int32 GetCurrentConcurrency() const { return CurrentConcurrency.load(std::memory_order_relaxed); }

protected:
	class FScheduledWork : public IQueuedWork, public IExecutionResource
	{
	public:
		CORE_API FScheduledWork();

		FScheduledWork(const FScheduledWork&) = delete;
		FScheduledWork& operator=(const FScheduledWork&) = delete;

		FScheduledWork(const FScheduledWork&&) = delete;
		FScheduledWork& operator=(const FScheduledWork&&) = delete;

		CORE_API ~FScheduledWork() override;
		
		const TCHAR * GetDebugName() const final
		{
			// return DebugName of innerwork, if any
			if ( Work )
			{
				return Work->GetDebugName();
			}
			else
			{
				return TEXT("FScheduledWork");
			}
		}
	protected:
		virtual void DoThreadedWork() override
		{
			{
				// Add this object as an execution context that can be retrieved via
				// FExecutionResourceContext::Get() if a task needs to hold on the
				// resources acquired (i.e. Concurrency Limit, Memory Pressure, etc...)
				// longer than for the DoThreadedWork() scope.
				FExecutionResourceContextScope ExecutionContextScope(this);

				Work->DoThreadedWork();
			}

			Release();
		}
	private:
		void AddRef() const override
		{
			NumRefs.Increment();
		}

		FReturnedRefCountValue Release() const override
		{
			uint32 Refs = uint32(NumRefs.Decrement());

 			// When the last ref is released, we call the schedule function of the parent pool
			// so that OnUnschedule can release any resources acquired by the OnSchedule function and
			// the scheduling of the next work items can proceed.
			if (Refs == 0)
			{
				ParentPool->Schedule(const_cast<FScheduledWork*>(this));
			}

			return FReturnedRefCountValue{ Refs };
		}

		FReturnedRefCountValue GetRefCount() const override
		{
			return FReturnedRefCountValue{ static_cast<uint32>(NumRefs.GetValue()) };
		}

		void Assign(FQueuedThreadPoolWrapper* InParentPool, IQueuedWork* InWork, EQueuedWorkPriority InPriority)
		{
			check(NumRefs.GetValue() == 0);
			ParentPool = InParentPool;
			Work = InWork;
			Priority = InPriority;
			AddRef();
		}

		void Abandon() override
		{
			Work->Abandon();

			Release();
		}

		EQueuedWorkFlags GetQueuedWorkFlags() const override
		{
			return Work->GetQueuedWorkFlags();
		}

		int64 GetRequiredMemory() const override
		{
			return Work->GetRequiredMemory();
		}

		IQueuedWork* GetInnerWork() const
		{
			return Work;
		}

		EQueuedWorkPriority GetPriority() const
		{
			return Priority;
		}

		void Reset()
		{
			Work = nullptr;
		}

		mutable FThreadSafeCounter NumRefs;
		friend class FQueuedThreadPoolWrapper;
		FQueuedThreadPoolWrapper* ParentPool;
	protected:
		IQueuedWork* Work;
	private:
		EQueuedWorkPriority Priority;
	};

	// A critical section is used since we need reentrancy support from the same thread
	FCriticalSection Lock;
	FThreadPoolPriorityQueue QueuedWork;

	// Can be overriden to dynamically control the maximum concurrency
	virtual int32 GetMaxConcurrency() const { return MaxConcurrency.load(std::memory_order_relaxed); }
	
	// Can be overriden to know when work has been scheduled.
	virtual void OnScheduled(const IQueuedWork*) {}
	
	// Can be overriden to know when work has been unscheduled.
	virtual void OnUnscheduled(const IQueuedWork*) {}

	// Can be overriden to allocate a more specialized version if needed.
	virtual FScheduledWork* AllocateScheduledWork() { return new FScheduledWork(); }

	// Can be overridden to ignore specific priority levels if required.
	CORE_API virtual bool CanSchedule(EQueuedWorkPriority Priority) const;
private:
	CORE_API FScheduledWork* AllocateWork(IQueuedWork* InnerWork, EQueuedWorkPriority Priority);
	CORE_API bool Create(uint32 InNumQueuedThreads, uint32 StackSize, EThreadPriority ThreadPriority, const TCHAR* Name) override;
	CORE_API void Destroy() override;
	CORE_API void Schedule(FScheduledWork* Work = nullptr);
	CORE_API void ReleaseWorkNoLock(FScheduledWork* Work);
	CORE_API bool TryRetractWorkNoLock(EQueuedWorkPriority InPriority);
	
	TFunction<EQueuedWorkPriority(EQueuedWorkPriority)> PriorityMapper;

	FQueuedThreadPool* WrappedQueuedThreadPool;
	TArray<FScheduledWork*> WorkPool;
	TMap<IQueuedWork*, FScheduledWork*> ScheduledWork;
	std::atomic<int32> MaxConcurrency;
	int32 MaxTaskToSchedule;
	std::atomic<int32> CurrentConcurrency;
	EQueuedWorkPriority WrappedQueuePriority;
	bool bIsScheduling = false;
};

/** ThreadPool wrapper implementation allowing to schedule
  * up to MaxConcurrency tasks at a time making sub-partitioning
  * another thread-pool a breeze and allowing more fine-grained control
  * over scheduling by giving full control of task reordering.
  */
class FQueuedThreadPoolDynamicWrapper : public FQueuedThreadPoolWrapper
{
public:
	/**
	 * InWrappedQueuedThreadPool  Underlying thread pool to schedule task to.
	 * InMaxConcurrency           Maximum number of concurrent tasks allowed, -1 will limit concurrency to number of threads available in the underlying thread pool.
	 * InPriorityMapper           Thread-safe function used to map any priority from this Queue to the priority that should be used when scheduling the task on the underlying thread pool.
	 */
	FQueuedThreadPoolDynamicWrapper(FQueuedThreadPool* InWrappedQueuedThreadPool, int32 InMaxConcurrency = -1, TFunction<EQueuedWorkPriority(EQueuedWorkPriority)> InPriorityMapper = DefaultQueuedWorkPriorityWrapper)
		: FQueuedThreadPoolWrapper(InWrappedQueuedThreadPool, InMaxConcurrency, InPriorityMapper)
	{
	}

	void AddQueuedWork(IQueuedWork* InQueuedWork, EQueuedWorkPriority InPriority = EQueuedWorkPriority::Normal) override
	{
		// Override priority to make sure all elements are in the same buckets and can then be sorted all together.
		FQueuedThreadPoolWrapper::AddQueuedWork(InQueuedWork, EQueuedWorkPriority::Normal);
	}

	/**
	 * Apply sort predicate to reorder the queued tasks
	 */
	void Sort(TFunctionRef<bool(const IQueuedWork* Lhs, const IQueuedWork* Rhs)> Predicate)
	{
		FScopeLock ScopeLock(&Lock);
		QueuedWork.Sort(EQueuedWorkPriority::Normal, Predicate);
	}
};

/** ThreadPool wrapper implementation allowing to schedule thread-pool tasks on the task graph.
  */
class UE_DEPRECATED(5.8, "FQueuedThreadPoolTaskGraphWrapper is obsolete and will be removed") FQueuedThreadPoolTaskGraphWrapper final : public FQueuedThreadPool
{
public:
	/**
	 * InPriorityMapper           Thread-safe function used to map any priority from this Queue to the priority that should be used when scheduling the task on the task graph.
	 */
	FQueuedThreadPoolTaskGraphWrapper(TFunction<ENamedThreads::Type (EQueuedWorkPriority)> InPriorityMapper = nullptr)
		: TaskCount(0)
		, bIsExiting(0)
	{
		if (InPriorityMapper)
		{
			PriorityMapper = InPriorityMapper;
		}
		else
		{
			PriorityMapper = [this](EQueuedWorkPriority InPriority) { return GetDefaultPriorityMapping(InPriority); };
		}
	}

	/**
	 * InDesiredThread           The task-graph desired thread and priority.
	 */
	FQueuedThreadPoolTaskGraphWrapper(ENamedThreads::Type InDesiredThread)
		: TaskCount(0)
		, bIsExiting(0)
	{
		PriorityMapper = [InDesiredThread](EQueuedWorkPriority InPriority) { return InDesiredThread; };
	}

	~FQueuedThreadPoolTaskGraphWrapper()
	{
		Destroy();
	}
private:
	void AddQueuedWork(IQueuedWork* InQueuedWork, EQueuedWorkPriority InPriority = EQueuedWorkPriority::Normal) override
	{
		check(bIsExiting == false);
		TaskCount++;
		FFunctionGraphTask::CreateAndDispatchWhenReady(
			[this, InQueuedWork](ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
			{
				FMemMark Mark(FMemStack::Get());
				InQueuedWork->DoThreadedWork();
				OnTaskCompleted(InQueuedWork);
			},
			QUICK_USE_CYCLE_STAT(FQueuedThreadPoolTaskGraphWrapper, STATGROUP_ThreadPoolAsyncTasks),
			nullptr,
			PriorityMapper(InPriority)
		);
	}

	bool RetractQueuedWork(IQueuedWork* InQueuedWork) override
	{
		// The task graph doesn't support retraction for now
		return false;
	}

	void OnTaskCompleted(IQueuedWork* InQueuedWork)
	{
		if (--TaskCount == 0 && bIsExiting)
		{
			Finished.Notify();
		}
	}

	int32 GetNumThreads() const override
	{
		return FTaskGraphInterface::Get().GetNumWorkerThreads();
	}

	ENamedThreads::Type GetDefaultPriorityMapping(EQueuedWorkPriority InQueuedWorkPriority)
	{
		ENamedThreads::Type DesiredThread = ENamedThreads::AnyNormalThreadNormalTask;
		if (InQueuedWorkPriority > EQueuedWorkPriority::Normal)
		{
			DesiredThread = ENamedThreads::AnyBackgroundThreadNormalTask;
		}
		else if (InQueuedWorkPriority < EQueuedWorkPriority::Normal)
		{
			DesiredThread = ENamedThreads::AnyHiPriThreadNormalTask;
		}
		return DesiredThread;
	}
protected:
	bool Create(uint32 InNumQueuedThreads, uint32 StackSize, EThreadPriority ThreadPriority, const TCHAR* Name) override
	{
		return true;
	}

	void Destroy() override
	{
		bIsExiting = true;

		if (TaskCount != 0)
		{
			Finished.Wait();
		}
	}
private:
	TFunction<ENamedThreads::Type (EQueuedWorkPriority)> PriorityMapper;
	TAtomic<uint32>       TaskCount;
	TAtomic<bool>         bIsExiting;
	UE::FManualResetEvent Finished;
};

/** ThreadPool wrapper implementation allowing to schedule thread-pool tasks on the the low level backend which is also used by the taskgraph.
*/
class FQueuedLowLevelThreadPool final : public FQueuedThreadPool
{
	/* Internal data of the scheduler used for cancellation */
	struct FQueuedWorkInternalData : TConcurrentLinearObject<FQueuedWorkInternalData, FTaskGraphBlockAllocationTag>, IQueuedWorkInternalData
	{
		LowLevelTasks::FTask Task;

		virtual bool Retract()
		{
			return Task.TryCancel();
		}
	};
public:
	/**
	* InPriorityMapper           Thread-safe function used to map any priority from this Queue to the priority that should be used when scheduling the task on the underlying thread pool.
	**/
	FQueuedLowLevelThreadPool(TFunction<EQueuedWorkPriority(EQueuedWorkPriority)> InPriorityMapper = DefaultQueuedWorkPriorityWrapper, LowLevelTasks::FScheduler* InScheduler = &LowLevelTasks::FScheduler::Get()) 
		: Scheduler(InScheduler), PriorityMapper(InPriorityMapper)
	{
	}

	~FQueuedLowLevelThreadPool()
	{
		Destroy();
	}

	void* operator new(size_t size)
	{
		return FMemory::Malloc(size, 128);
	}

	void operator delete(void* ptr)
	{
		FMemory::Free(ptr);
	}

	/**
	*  Queued task are not scheduled against the wrapped thread-pool until resumed
	*/
	void Pause()
	{
		bIsPaused = true;
	}

	/**
	*  Resume a specified amount of queued work, or -1 to unpause.
	*/
	void Resume(int32 InNumQueuedWork = -1)
	{
		for (uint32 i = 0; i < uint32(InNumQueuedWork); i++)
		{
			FQueuedWorkInternalData* QueuedWork = Dequeue();
			if (!QueuedWork)
			{
				break;
			}
			TaskCount.fetch_add(1, std::memory_order_acquire);
			verifySlow(Scheduler->TryLaunch(QueuedWork->Task, LowLevelTasks::EQueuePreference::GlobalQueuePreference));
		}

		if (InNumQueuedWork == -1)
		{
			bIsPaused = false;
		}

		bool bWakeUpWorker = true;
		ScheduleTasks(bWakeUpWorker);
	}

private:
	void ScheduleTasks(bool &bWakeUpWorker)
	{
		while (!bIsPaused)
		{
			FQueuedWorkInternalData* QueuedWork = Dequeue();
			if (QueuedWork)
			{
				verifySlow(Scheduler->TryLaunch(QueuedWork->Task, bWakeUpWorker ? LowLevelTasks::EQueuePreference::GlobalQueuePreference : LowLevelTasks::EQueuePreference::LocalQueuePreference, bWakeUpWorker));
				TaskCount.fetch_add(1, std::memory_order_acquire);
				bWakeUpWorker = true;
			}
			else
			{
				break;
			}
		}
	}

	void FinalizeExecution()
	{
		if (TaskCount.fetch_sub(1, std::memory_order_release) == 1 && bIsExiting)
		{
			Finished.Notify();
		}
		else
		{
			bool bWakeUpWorker = false;
			ScheduleTasks(bWakeUpWorker);
		}
	}

	void AddQueuedWork(IQueuedWork* InQueuedWork, EQueuedWorkPriority InPriority = EQueuedWorkPriority::Normal) override
	{
		check(bIsExiting == false);

		FQueuedWorkInternalData* QueuedWorkInternalData = new FQueuedWorkInternalData();
		InQueuedWork->InternalData = QueuedWorkInternalData;
		EQueuedWorkPriority Priority = PriorityMapper(InPriority);

		LowLevelTasks::ETaskPriority TaskPriorityMapper[int(EQueuedWorkPriority::Count)] = { LowLevelTasks::ETaskPriority::High, LowLevelTasks::ETaskPriority::High, LowLevelTasks::ETaskPriority::BackgroundHigh, LowLevelTasks::ETaskPriority::BackgroundNormal, LowLevelTasks::ETaskPriority::BackgroundLow, LowLevelTasks::ETaskPriority::BackgroundLow };
		LowLevelTasks::ETaskPriority TaskPriority = TaskPriorityMapper[int(Priority)];
		LowLevelTasks::ETaskFlags Flags = (InQueuedWork->GetQueuedWorkFlags() & EQueuedWorkFlags::DoNotRunInsideBusyWait) == EQueuedWorkFlags::None ? LowLevelTasks::ETaskFlags::DefaultFlags : (LowLevelTasks::ETaskFlags::DefaultFlags & ~LowLevelTasks::ETaskFlags::AllowBusyWaiting);

		QueuedWorkInternalData->Task.Init(TEXT("FQueuedLowLevelThreadPoolTask"), TaskPriority, [InQueuedWork, InternalData = InQueuedWork->InternalData, Deleter = LowLevelTasks::TDeleter<FQueuedLowLevelThreadPool, &FQueuedLowLevelThreadPool::FinalizeExecution>{ this }]
		{
			FMemMark Mark(FMemStack::Get());
			InQueuedWork->DoThreadedWork();
		}, Flags);

		if (!bIsPaused)
		{
			TaskCount.fetch_add(1, std::memory_order_acquire);
			verifySlow(Scheduler->TryLaunch(QueuedWorkInternalData->Task, LowLevelTasks::EQueuePreference::GlobalQueuePreference));
		}
		else
		{
			Enqueue(Priority, QueuedWorkInternalData);
		}
	}

	bool RetractQueuedWork(IQueuedWork* InQueuedWork) override
	{
		bool bCancelled = false;
		if(InQueuedWork->InternalData.IsValid())
		{
			bCancelled = InQueuedWork->InternalData->Retract();
			InQueuedWork->InternalData = nullptr;
		}

		bool bWakeUpWorker = true;
		ScheduleTasks(bWakeUpWorker);	
		return bCancelled;
	}

	int32 GetNumThreads() const override
	{
		return Scheduler->GetNumWorkers();
	}

protected:
	bool Create(uint32 InNumQueuedThreads, uint32 InStackSize, EThreadPriority InThreadPriority, const TCHAR* InName) override
	{
		return true;
	}

	void Destroy() override
	{
		bIsExiting = true;

		while (true)
		{
			FQueuedWorkInternalData* QueuedWork = Dequeue();
			if (!QueuedWork)
			{
				break;
			}

			verify(QueuedWork->Retract());
			TaskCount++;
			verifySlow(Scheduler->TryLaunch(QueuedWork->Task, LowLevelTasks::EQueuePreference::GlobalQueuePreference));
		}

		if (TaskCount != 0)
		{
			Finished.Wait();
		}
	}

private:
	FAAArrayQueue<FQueuedWorkInternalData> PendingWork[int32(EQueuedWorkPriority::Count)];

	inline FQueuedWorkInternalData* Dequeue()
	{
		for (int32 i = 0; i < int32(EQueuedWorkPriority::Count); i++)
		{
			FQueuedWorkInternalData* QueuedWork = PendingWork[i].dequeue();
			if (QueuedWork)
			{
				return QueuedWork;
			}
		}
		return nullptr;
	}

	inline void Enqueue(EQueuedWorkPriority Priority, FQueuedWorkInternalData* Item)
	{
		PendingWork[int32(Priority)].enqueue(Item);
	}

	LowLevelTasks::FScheduler* Scheduler = nullptr;
	TFunction<EQueuedWorkPriority(EQueuedWorkPriority)> PriorityMapper;

	std::atomic_uint TaskCount{0};
	std::atomic_bool bIsExiting{false};
	std::atomic_bool bIsPaused{false};
	UE::FManualResetEvent Finished;
};
