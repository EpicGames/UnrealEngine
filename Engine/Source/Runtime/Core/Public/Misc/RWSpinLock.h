// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformProcess.h"
#include "Math/NumericLimits.h"

#include <atomic>

namespace UE
{
	// A read-write lock that doesn't put the thread into a WAIT state but instead repeatedly tries to acquire the lock.
	// WARNING: Should be used only for very short locks.
	// Use with `TRWScopeLock`, `TWriteScopeLock` or `TReadScopeLock`.
	// The SizeType template type dictates the space taken by the spinlock but also its maximum number of possible concurrent readers (i.e. 254 for uint8, etc...).
	// Read locks support recursion. Write locks don't support recursion even if coming from the same thread currently owning the write lock.
	template <typename SizeType = uint32>
	class TRWSpinLock
	{
	public:
		static_assert(TNumericLimits<SizeType>::Min() >= 0, "Type must be unsigned");
		UE_NONCOPYABLE(TRWSpinLock);

		TRWSpinLock() = default;

		bool TryWriteLock()
		{
			SizeType Expected = 0;
			return Lock.compare_exchange_strong(Expected, TNumericLimits<SizeType>::Max(), std::memory_order_acquire, std::memory_order_relaxed);
		}

		void WriteLock() 
		{
			while (!TryWriteLock())
			{
				// Reduce contention by doing a simple relaxed read to see if we have a chance of being able to lock.
				while (Lock.load(std::memory_order_relaxed) != 0)
				{
					FPlatformProcess::Yield();
				}
			}
		}

		void WriteUnlock() 
		{
			Lock.store(0, std::memory_order_release);
		}

		bool TryReadLock()
		{
			SizeType LocalValue = Lock.load(std::memory_order_relaxed);
			// Check to make sure we don't already have a write lock or that we've not reached the limit of reader locks.
			if (LocalValue >= TNumericLimits<SizeType>::Max() - 1)
			{
				return false;
			}

			return Lock.compare_exchange_strong(LocalValue, LocalValue + 1, std::memory_order_acquire, std::memory_order_relaxed);
		}

		void ReadUnlock() 
		{
			Lock.fetch_sub(1, std::memory_order_release);
		}

		void ReadLock()
		{
			while (!TryReadLock())
			{
				FPlatformProcess::Yield();
			}
		}

	private:
		std::atomic<SizeType> Lock = 0;
	};

	using FRWSpinLock = TRWSpinLock<uint32>;
}
