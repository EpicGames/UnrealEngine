// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RangeAllocator.h: Memory allocator that sub-allocates from ranges of memory
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"

#define RANGE_ALLOCATOR_RECORD_STATS !UE_BUILD_SHIPPING

/*
 * This allocator has a list of chunks (fixed size committed memory regions) and each
 * chunk keeps a list of free ranges. Allocations are served by finding a large enough
 * free range and sub-allocating from it. Freed allocations become free ranges and are
 * combined with adjacent free ranges if possible. It is meant to be a faster version
 * of TGenericGrowableAllocator but imposes some restrictions on size and alignment.
 * The restrictions are that alignments must be power of two, ChunkSize must be a multiple
 * of MinAlignment, and the quotient of dividing ChunkSize by MinAlignment must be
 * representable by a 16-bit unsigned integer. Note that allocations smaller than
 * MinAlignment will be padded to MinAlignment and it cannot allocate larger than
 * ChunkSize. It is advised to use multiple instances of range allocators where each
 * handles a range of allocation sizes (e.g. three allocators for allocations smaller
 * than 1 KB, between 1 KB and 128 KB, and between 128 KB and 1 MB, respectively). Doing
 * so usually lead to better performance and less fragmentation.
 * 
 * To use this allocator, implement your own FAllocatorModifier to customize behaviors
 * and handle platform specific stuff. Your implementation should at minimum contain the
 * members listed below.
 * 
 * class FAllocatorModifier
 * {
 *   static constexpr uint32 ChunkSize = <...>;
 *   static constexpr uint32 MinAlignment = <...>;
 *
 *   // The type of "pointer" returned by this allocator
 *   typedef <...> FPointerType;
 *   typedef <...> FConstPointerType;
 *
 *   // Contains information needed to represent a chunk
 *   struct FChunkModifier
 *   {
 *     // You may allocate the actual backing memory here and store its base address
 *     FChunkModifier(const FAllocatorModifier& Allocator);
 * 
 *     // Custom cleanup such as releasing backing memory
 *     ~FChunkModifier();
 * 
 *     // Reinterpret the offset in chunk such as adding it to a base address
 *     SIZE_T AdjustAllocation(SIZE_T OffsetInChunk) [const];
 * 
 *     // Whether this chunk is valid
 *     bool IsValid() const;
 * 
 *     // Whether Addr is in this chunk
 *     bool Contains(FConstPointerType Addr) const;
 *   };
 * 
 *   // Contains information needed to represent an allocation
 *   struct FAllocInfoModifier
 *   {
 *     // Initialize custom members. Ptr is the output of AdjustAllocation
 *     void InitCustomInfo(const FChunkModifier& Chunk, SIZE_T Ptr);
 *   };
 * 
 *   // Convert AdjustAllocation output to the pointer type such as a simple type cast
 *   static FPointerType MakePointer(SIZE_T Ptr);
 * 
 *   // Custom assert implementation
 *   static void Check(bool bOk);
 * };
 * 
 * Once the modifier is implemented, feed it to TRangeAllocator as a template argument.
 * You may derive from TRangeAllocator<FAllocatorModifier> to do further customization
 * like adding methods that access protected TRangeAllocator member variables.
 */
 template <typename FAllocatorModifier>
class TRangeAllocator : public FAllocatorModifier
{
protected:
	using FRangeAllocator = TRangeAllocator<FAllocatorModifier>;
	using FPointerType = typename FAllocatorModifier::FPointerType;
	using FConstPointerType = typename FAllocatorModifier::FConstPointerType;
	using FChunkModifier = typename FAllocatorModifier::FChunkModifier;
	using FAllocInfoModifier = typename FAllocatorModifier::FAllocInfoModifier;

	using FAllocatorModifier::Check;
	using FAllocatorModifier::MakePointer;

	class FChunk : public FChunkModifier
	{
		using FChunkModifier::AdjustAllocation;

		struct FRange
		{
			uint16 OffsetMultiplier;
			uint16 SizeMultiplier;

			FRange(uint16 InOffsetMultiplier, uint16 InSizeMultiplier)
				: OffsetMultiplier(InOffsetMultiplier)
				, SizeMultiplier(InSizeMultiplier)
			{}
		};

		TArray<FRange> FreeRanges;

	public:
		union
		{
			int32 InfoIndex;
			int32 NextFreeChunkSlot;
		};

		using FChunkModifier::IsValid;
		using FChunkModifier::Contains;

		FChunk(const FRangeAllocator& Allocator)
			: FChunkModifier(Allocator)
			, InfoIndex(INDEX_NONE)
		{
			FreeRanges.Add(FRange(0, Allocator.ChunkSize / MinAlignment));
		}

		SIZE_T Alloc(uint32 InSize, uint32 InAlignment, uint16& InOutMaxFreeRangeSize, uint16& OutOffset, uint16& OutSize)
		{
			bool bMaxRangeSizeDirty = false;
			uint16 NewMaxFreeRangeSize = 0;
			Check(InSize / MinAlignment <= UINT16_MAX);
			const uint16 Size = uint16(InSize / MinAlignment);
			Check(InAlignment >= MinAlignment && InAlignment / MinAlignment <= UINT16_MAX);
			const uint16 Alignment = uint16(InAlignment / MinAlignment);
			SIZE_T OffsetInChunk = 0;

			// Find a large enough free range and sub-allocate from it
			Check(FreeRanges.Num() > 0);
			int32 Index = 0;
			for (; Index < FreeRanges.Num(); ++Index)
			{
				FRange& Range = FreeRanges[Index];
				const uint16 Start = Range.OffsetMultiplier;
				const uint16 AlignedStart = Align(Start, Alignment);
				const uint16 Padding = AlignedStart - Start;
				Check((uint32)Padding + (uint32)Size <= (uint32)UINT16_MAX);
				const uint16 RequiredSize = Padding + Size;
				if (Range.SizeMultiplier >= RequiredSize)
				{
					const uint16 RemainingSize = Range.SizeMultiplier - RequiredSize;
					bMaxRangeSizeDirty = Range.SizeMultiplier == InOutMaxFreeRangeSize;
					OffsetInChunk = (SIZE_T)AlignedStart * MinAlignment;
					OutOffset = Range.OffsetMultiplier;
					OutSize = RequiredSize;
					if (!RemainingSize)
					{
						FreeRanges.RemoveAt(Index, EAllowShrinking::No);
					}
					else
					{
						Range.OffsetMultiplier += RequiredSize;
						Range.SizeMultiplier = RemainingSize;
						NewMaxFreeRangeSize = FMath::Max(NewMaxFreeRangeSize, Range.SizeMultiplier);
					}

					if (!bMaxRangeSizeDirty)
					{
						return AdjustAllocation(OffsetInChunk);
					}
					++Index;
					break;
				}
				else
				{
					NewMaxFreeRangeSize = FMath::Max(NewMaxFreeRangeSize, Range.SizeMultiplier);
				}
			}

			// If MaxFreeRangeSize is dirty, loop through all the free ranges to figure out the new max
			if (bMaxRangeSizeDirty)
			{
				for (; Index < FreeRanges.Num(); ++Index)
				{
					const FRange& Range = FreeRanges[Index];
					NewMaxFreeRangeSize = FMath::Max(NewMaxFreeRangeSize, Range.SizeMultiplier);
				}
				InOutMaxFreeRangeSize = NewMaxFreeRangeSize;
				return AdjustAllocation(OffsetInChunk);
			}

			// No free range is large enough, return nullptr
			return 0;
		}

		void Free(uint16 Offset, uint16 Size, uint16& InOutMaxFreeRangeSize)
		{
			const int32 UpperBoundIndex = Algo::UpperBoundBy(FreeRanges, Offset, [](const FRange& Range) { return Range.OffsetMultiplier; });
			Check(UpperBoundIndex >= 0 && UpperBoundIndex <= FreeRanges.Num());
			Check(UpperBoundIndex == FreeRanges.Num() || Offset < FreeRanges[UpperBoundIndex].OffsetMultiplier);
			const int32 LowerBoundIndex = UpperBoundIndex - 1;
			Check(LowerBoundIndex >= -1 && LowerBoundIndex < FreeRanges.Num());
			Check(LowerBoundIndex == -1 || FreeRanges[LowerBoundIndex].OffsetMultiplier < Offset);
			bool bInsertRange = true;
			// Try merge with left
			if (LowerBoundIndex >= 0 && Offset == FreeRanges[LowerBoundIndex].OffsetMultiplier + FreeRanges[LowerBoundIndex].SizeMultiplier)
			{
				Check((uint32)Size + FreeRanges[LowerBoundIndex].SizeMultiplier <= UINT16_MAX);
				FreeRanges[LowerBoundIndex].SizeMultiplier += Size;
				Offset = FreeRanges[LowerBoundIndex].OffsetMultiplier;
				Size = FreeRanges[LowerBoundIndex].SizeMultiplier;
				bInsertRange = false;
			}
			// Try merge with right
			if (UpperBoundIndex < FreeRanges.Num() && Offset + Size == FreeRanges[UpperBoundIndex].OffsetMultiplier)
			{
				if (bInsertRange)
				{
					FreeRanges[UpperBoundIndex].OffsetMultiplier = Offset;
					Check((uint32)Size + FreeRanges[UpperBoundIndex].SizeMultiplier <= UINT16_MAX);
					FreeRanges[UpperBoundIndex].SizeMultiplier += Size;
					Size = FreeRanges[UpperBoundIndex].SizeMultiplier;
					bInsertRange = false;
				}
				else
				{
					FreeRanges[LowerBoundIndex].SizeMultiplier += FreeRanges[UpperBoundIndex].SizeMultiplier;
					Size = FreeRanges[LowerBoundIndex].SizeMultiplier;
					FreeRanges.RemoveAt(UpperBoundIndex, EAllowShrinking::No);
				}
			}
			// No merging, add a new free range
			if (bInsertRange)
			{
				FreeRanges.Insert(FRange(Offset, Size), UpperBoundIndex);
				Check(UpperBoundIndex + 1 == FreeRanges.Num() || Offset < FreeRanges[UpperBoundIndex + 1].OffsetMultiplier);
				Check(UpperBoundIndex == 0 || FreeRanges[UpperBoundIndex - 1].OffsetMultiplier < Offset);
			}
			// Keep MaxFreeRangeSize up-to-date
			InOutMaxFreeRangeSize = FMath::Max(InOutMaxFreeRangeSize, Size);
		}
	};

	struct FFreeChunkInfo
	{
		uint16 MaxFreeRangeSize;
		uint16 ChunkIndex;

		FFreeChunkInfo(uint16 InMaxFreeRangeSize, uint16 InChunkIndex)
			: MaxFreeRangeSize(InMaxFreeRangeSize)
			, ChunkIndex(InChunkIndex)
		{}
	};

	void RemoveFreeChunkInfo(int32 InfoIndex)
	{
		if (InfoIndex != FreeChunkInfos.Num() - 1)
		{
			Chunks[FreeChunkInfos.Last().ChunkIndex].InfoIndex = InfoIndex;
		}
		FreeChunkInfos.RemoveAtSwap(InfoIndex, EAllowShrinking::No);
	}

	void FreeChunk(uint32 ChunkIndex)
	{
#if RANGE_ALLOCATOR_RECORD_STATS
		SizeTotal -= ChunkSize;
#endif
		const int32 InfoIndex = Chunks[ChunkIndex].InfoIndex;
		if (InfoIndex != INDEX_NONE)
		{
			RemoveFreeChunkInfo(InfoIndex);
		}
		Chunks[ChunkIndex].~FChunk();
		Chunks[ChunkIndex].NextFreeChunkSlot = FreeChunkSlotHead;
		FreeChunkSlotHead = ChunkIndex;
	}

#if RANGE_ALLOCATOR_RECORD_STATS
	SIZE_T SizeTotal = 0; // Total amount allocated in bytes including slack
	SIZE_T SizeUsed = 0;  // Size of active allocations in bytes
#endif

	const uint16 MinAllocSize;
	int32 FreeChunkSlotHead;
	TArray<FFreeChunkInfo> FreeChunkInfos;
	TArray<FChunk> Chunks;
	mutable FCriticalSection CS;

public:
	using FAllocatorModifier::ChunkSize;
	using FAllocatorModifier::MinAlignment;

	struct FAllocInfo : public FAllocInfoModifier
	{
		uint16 RangeOffset;
		uint16 RangeSize;
		uint32 ChunkIndex;
	};

	template <typename... ArgTypes>
	TRangeAllocator(uint32 InMinAllocSize, ArgTypes&&... Args)
		: FAllocatorModifier(Forward<ArgTypes>(Args)...)
		, MinAllocSize(uint16(InMinAllocSize / MinAlignment))
		, FreeChunkSlotHead(INDEX_NONE)
	{}

	FPointerType Alloc(uint32 Size, uint32 Alignment, FAllocInfo& OutAllocInfo)
	{
		FScopeLock Lock(&CS);

		Alignment = Align(FMath::Max(Alignment, MinAlignment), MinAlignment);
		Check(FMath::IsPowerOfTwo(Alignment));
		Size = Align(Size, Alignment);
		Check(Size <= ChunkSize);
		Check(Size >= MinAllocSize * MinAlignment);

		const uint16 NumFreeChunks = (uint16)FreeChunkInfos.Num();
		TArray<uint16, TInlineAllocator<24>> MayFitInfoIndices;

		// Find a sure fit first
		for (uint16 Index = 0; Index < NumFreeChunks; ++Index)
		{
			FFreeChunkInfo& Info = FreeChunkInfos[Index];
			if (Size + Alignment - 1 <= (uint32)Info.MaxFreeRangeSize * MinAlignment)
			{
				FChunk& Chunk = Chunks[Info.ChunkIndex];
				SIZE_T Ptr = Chunk.Alloc(Size, Alignment, Info.MaxFreeRangeSize, OutAllocInfo.RangeOffset, OutAllocInfo.RangeSize);
				Check(Ptr != 0);
#if RANGE_ALLOCATOR_RECORD_STATS
				SizeUsed += OutAllocInfo.RangeSize * MinAlignment;
#endif
				OutAllocInfo.ChunkIndex = Info.ChunkIndex;
				OutAllocInfo.InitCustomInfo(Chunk, Ptr);
				if (Info.MaxFreeRangeSize < MinAllocSize)
				{
					Chunk.InfoIndex = INDEX_NONE;
					RemoveFreeChunkInfo(Index);
				}
				return MakePointer(Ptr);
			}
			else if (Size <= (uint32)Info.MaxFreeRangeSize * MinAlignment)
			{
				MayFitInfoIndices.Add(Index);
			}
		}

		// Check may fits
		for (int32 Index = 0; Index < MayFitInfoIndices.Num(); ++Index)
		{
			const int32 InfoIndex = MayFitInfoIndices[Index];
			FFreeChunkInfo& Info = FreeChunkInfos[InfoIndex];
			FChunk& Chunk = Chunks[Info.ChunkIndex];
			SIZE_T Ptr = Chunk.Alloc(Size, Alignment, Info.MaxFreeRangeSize, OutAllocInfo.RangeOffset, OutAllocInfo.RangeSize);
			if (Ptr != 0)
			{
#if RANGE_ALLOCATOR_RECORD_STATS
				SizeUsed += OutAllocInfo.RangeSize * MinAlignment;
#endif
				OutAllocInfo.ChunkIndex = Info.ChunkIndex;
				OutAllocInfo.InitCustomInfo(Chunk, Ptr);
				if (Info.MaxFreeRangeSize < MinAllocSize)
				{
					Chunk.InfoIndex = INDEX_NONE;
					RemoveFreeChunkInfo(InfoIndex);
				}
				return MakePointer(Ptr);
			}
		}

		// Alloc from a new chunk
		uint16 MaxFreeRangeSize = ChunkSize / MinAlignment;
		Check(Chunks.Num() < UINT16_MAX);
		int32 ChunkIndex;
		if (FreeChunkSlotHead != INDEX_NONE)
		{
			ChunkIndex = FreeChunkSlotHead;
			FreeChunkSlotHead = Chunks[FreeChunkSlotHead].NextFreeChunkSlot;
		}
		else
		{
			ChunkIndex = Chunks.AddUninitialized();
		}
		FChunk& Chunk = Chunks[ChunkIndex];
		::new ((void*)&Chunk) FChunk(*this);
		SIZE_T Ptr = Chunk.Alloc(Size, Alignment, MaxFreeRangeSize, OutAllocInfo.RangeOffset, OutAllocInfo.RangeSize);
		Check(Ptr != 0);
#if RANGE_ALLOCATOR_RECORD_STATS
		SizeTotal += ChunkSize;
		SizeUsed += OutAllocInfo.RangeSize * MinAlignment;
#endif
		OutAllocInfo.ChunkIndex = ChunkIndex;
		OutAllocInfo.InitCustomInfo(Chunk, Ptr);
		if (MaxFreeRangeSize >= MinAllocSize)
		{
			Chunk.InfoIndex = FreeChunkInfos.Add(FFreeChunkInfo(MaxFreeRangeSize, (uint16)ChunkIndex));
		}
		return MakePointer(Ptr);
	}

	void Free(const FAllocInfo& AllocInfo)
	{
		FScopeLock Lock(&CS);

		const int32 InfoIndex = Chunks[AllocInfo.ChunkIndex].InfoIndex;
		uint16 NewMaxFreeRangeSize = InfoIndex == INDEX_NONE ? 0 : FreeChunkInfos[InfoIndex].MaxFreeRangeSize;

		Chunks[AllocInfo.ChunkIndex].Free(AllocInfo.RangeOffset, AllocInfo.RangeSize, NewMaxFreeRangeSize);
#if RANGE_ALLOCATOR_RECORD_STATS
		SizeUsed -= AllocInfo.RangeSize * MinAlignment;
#endif
		if (NewMaxFreeRangeSize == ChunkSize / MinAlignment)
		{
			FreeChunk(AllocInfo.ChunkIndex);
		}
		else if (InfoIndex == INDEX_NONE)
		{
			if (NewMaxFreeRangeSize >= MinAllocSize)
			{
				Check(AllocInfo.ChunkIndex <= UINT16_MAX);
				Chunks[AllocInfo.ChunkIndex].InfoIndex = FreeChunkInfos.Add(FFreeChunkInfo(NewMaxFreeRangeSize, (uint16)AllocInfo.ChunkIndex));
			}
		}
		else
		{
			FreeChunkInfos[InfoIndex].MaxFreeRangeSize = NewMaxFreeRangeSize;
		}
	}

	bool Contains(FConstPointerType Addr) const
	{
		FScopeLock Lock(&CS);

		for (const FChunk& Chunk : Chunks)
		{
			if (Chunk.IsValid() && Chunk.Contains(Addr))
			{
				return true;
			}
		}
		return false;
	}
};
