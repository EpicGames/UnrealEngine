// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "Misc/DateTime.h"
#include "Stats/Stats.h"

class FConfigFile;
template<typename TTask> class FAsyncTask;

/**
 * Stores info relating to remote config files
 */
struct FRemoteConfigAsyncIOInfo
{
	/** Constructors */
	FRemoteConfigAsyncIOInfo() { }
	FRemoteConfigAsyncIOInfo(const TCHAR* InDefaultIniFile);

	/** Stores the contents of the remote config file */
	FString Buffer;

	/** Time stamp of the remote config file */
	FDateTime TimeStamp;

	/** Time at which the last read was initiated */
	double StartReadTime;

	/** Time at which the last write was initiated */
	double StartWriteTime;

	/** If the last read operation failed, this flag will be set to true */
	bool bReadIOFailed;

	/** true if this file has been processed */
	bool bWasProcessed;

	/** Name of the corresponding default config file */
	TCHAR DefaultIniFile[1024];
};


/**
 * Async task that handles the IO of a remote config file
 */
class FRemoteConfigAsyncWorker
{
public:

	/** Constructor. */
	FRemoteConfigAsyncWorker(const TCHAR* InFilename, FRemoteConfigAsyncIOInfo& InIOInfo, FString* InContents, bool bInIsRead);

	/** Performs the actual IO operations. */
	void DoWork();

	/** Returns true if the read IO operation succeeded. */
	bool IsReadSuccess() const;

	/** Returns the local IO info object */
	FRemoteConfigAsyncIOInfo& GetIOInfo();

	UE_FORCEINLINE_HINT TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FRemoteConfigAsyncWorker, STATGROUP_ThreadPoolAsyncTasks);
	}

	/** Indicates to the thread pool that this task is abandonable. */
	bool CanAbandon();

	/** Abandon routine */
	void Abandon();

private:

	/** Name of the remote config file. */
	TCHAR Filename[1024];

	/** Local copy of the IO info object, for thread safety. */
	FRemoteConfigAsyncIOInfo IOInfo;

	/** Contents to write out. */
	FString Contents;

	/** true if read operation, false if write. */
	bool bIsRead;
};


/**
 * Info for cached write tasks
 */
class FRemoteConfigAsyncCachedWriteTask
{
public:
	FRemoteConfigAsyncCachedWriteTask(const TCHAR* InFileName, FString* InContents)
	{
		Filename = InFileName;
		Contents = *InContents;
	}

	FString Filename;
	FString Contents;
};


/**
 * Manages async IO tasks for remote config files
 */
class FRemoteConfigAsyncTaskManager
{
public:

	/** Returns a reference to the global FRemoteConfigAsyncTaskManager object. */
	CORE_API static FRemoteConfigAsyncTaskManager* Get();

	/** Handles cached write tasks. */
	CORE_API void Tick();

	/** Add an async IO task to the queue and kick it off. */
	bool StartTask(const TCHAR* InFilename, const TCHAR* RemotePath, FRemoteConfigAsyncIOInfo& InIOInfo, FString* InContents, bool bInIsRead);

	/** Returns true if the task has completed */
	bool IsFinished(const TCHAR* InFilename);

	/** Returns true if the all tasks in the queue have completed (or, if the queue is empty). */
	bool AreAllTasksFinished(bool bDoRemoval);

	/** Safely retrieve the read data from the completed async task. */
	bool GetReadData(const TCHAR* InFilename, FRemoteConfigAsyncIOInfo& OutIOInfo);

private:

	/** Returns true if file exists in the cached task list. */
	bool FindCachedWriteTask(const TCHAR* InFilename, bool bCompareContents, const TCHAR* InContents);

	/** List of pending async IO operations */
	TMap<FString, FAsyncTask<FRemoteConfigAsyncWorker>* > PendingTasks;

	/** List of cached off write tasks waiting for currently pending tasks to complete. */
	TArray<FRemoteConfigAsyncCachedWriteTask> CachedWriteTasks;

	/** Object used for synchronization via a scoped lock. **/
	FCriticalSection SynchronizationObject;
};


/**
 * Manages remote config files.
 */
class FRemoteConfig
{
public:

	/** Constructor */
	FRemoteConfig();

	/** Returns a reference to the global FRemoteConfig object. */
	static FRemoteConfig* Get();

	/** Returns true if the specified config file has been flagged as being remote. */
	bool IsRemoteFile(const TCHAR* Filename);

	/** Returns true if the specified file is remote and still needs to be read. */
	bool ShouldReadRemoteFile(const TCHAR* Filename);

	/** Simple accessor function. */
	FRemoteConfigAsyncIOInfo* FindConfig(const TCHAR* Filename);

	/** Returns true if the task has completed. */
	bool IsFinished(const TCHAR* InFilename);

	/** Queues up a new async task for reading a remote config file. */
	bool Read(const TCHAR* GeneratedIniFile, const TCHAR* DefaultIniFile);

	/** Queues up a new async task for writing a remote config file. */
	bool Write(const TCHAR* Filename, FString& Contents);

	/** Waits on the async read if it hasn't finished yet... times out if the operation has taken too long. */
	void FinishRead(const TCHAR* Filename);

	/** Finishes all pending async IO tasks. */
	CORE_API static void Flush();

	/* Replaces chars used by the ini parser with "special chars". */
	CORE_API static FString ReplaceIniCharWithSpecialChar(const FString& Str);

	/* Replaces "special chars" that have been inserted to avoid problems with the ini parser with equivalent regular chars. */
	CORE_API static FString ReplaceIniSpecialCharWithChar(const FString& Str);
	
private:

	/** Creates the remote path string for the specified file. */
	FString GenerateRemotePath(const TCHAR* Filename);

	/** List of remote config file info. */
	TMap<FString, FRemoteConfigAsyncIOInfo> ConfigBuffers;

	/** Async IO operation timeout. */
	float Timeout;

	/** If true, remote ops are allowed. */
	bool bIsEnabled;

	/** true if the list of desired remote files has been filled out. */
	bool bHasCachedFilenames;

	/** List of desired remote files. */
	TArray<FString> CachedFileNames;
};


/** Returns true if no remote version of this config file exists and/or isn't being used. */
bool IsUsingLocalIniFile(const TCHAR* FilenameToLoad, const TCHAR* IniFileName);

/** Contains the logic for processing config files, local or remote. */
void ProcessIniContents(const TCHAR* FilenameToLoad, const TCHAR* IniFileName, FConfigFile* Config, bool bDoEmptyConfig, bool bDoCombine);

/** Returns the timestamp of the appropriate config file. */
FDateTime GetIniTimeStamp(const TCHAR* FilenameToLoad, const TCHAR* IniFileName);

/** Before overwriting the local file with the contents from the remote file, we save off a copy of the local file (if it exists). */
void MakeLocalCopy(const TCHAR* Filename);
