// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include <iterator>

/**
 * Pointer-like reverse iterator type.
 */
template <typename T>
struct TReversePointerIterator
{
	// This iterator type only supports the minimal functionality needed to support
	// C++ ranged-for syntax.  For example, it does not provide post-increment ++ nor ==.

	/**
	 * Constructor for TReversePointerIterator.
	 *
	 * @param  InPtr  A pointer to the location after the element being referenced.
	 *
	 * @note  Like std::reverse_iterator, this points to one past the element being referenced.
	 *        Therefore, for an array of size N starting at P, the begin iterator should be
	 *        constructed at (P+N) and the end iterator should be constructed at P.
	 */
	constexpr explicit TReversePointerIterator(T* InPtr UE_LIFETIMEBOUND)
		: Ptr(InPtr)
	{
	}

	constexpr inline T& operator*() const
	{
		return *(Ptr - 1);
	}

	constexpr inline TReversePointerIterator& operator++()
	{
		--Ptr;
		return *this;
	}

	constexpr inline bool operator!=(const TReversePointerIterator& Rhs) const
	{
		return Ptr != Rhs.Ptr;
	}

private:
	T* Ptr;
};

namespace UE::Core::Private
{
	template <typename RangeType>
	struct TReverseIterationAdapter
	{
		constexpr explicit TReverseIterationAdapter(RangeType& InRange UE_LIFETIMEBOUND)
			: Range(InRange)
		{
		}

		TReverseIterationAdapter(TReverseIterationAdapter&&) = delete;
		TReverseIterationAdapter& operator=(TReverseIterationAdapter&&) = delete;
		TReverseIterationAdapter& operator=(const TReverseIterationAdapter&) = delete;
		TReverseIterationAdapter(const TReverseIterationAdapter&) = delete;
		~TReverseIterationAdapter() = default;

		UE_FORCEINLINE_HINT constexpr auto begin() const UE_LIFETIMEBOUND
		{
			using std::rbegin;
			return rbegin(Range);
		}

		UE_FORCEINLINE_HINT constexpr auto end() const UE_LIFETIMEBOUND
		{
			using std::rend;
			return rend(Range);
		}

	private:
		RangeType& Range;
	};

	template <typename ElementType, std::size_t N>
	struct TReverseIterationAdapter<ElementType(&)[N]>
	{
		constexpr explicit TReverseIterationAdapter(ElementType* InArray UE_LIFETIMEBOUND)
			: Array(InArray)
		{
		}

		TReverseIterationAdapter(TReverseIterationAdapter&&) = delete;
		TReverseIterationAdapter& operator=(TReverseIterationAdapter&&) = delete;
		TReverseIterationAdapter& operator=(const TReverseIterationAdapter&) = delete;
		TReverseIterationAdapter(const TReverseIterationAdapter&) = delete;
		~TReverseIterationAdapter() = default;

		inline constexpr TReversePointerIterator<ElementType> begin() const UE_LIFETIMEBOUND
		{
			return TReversePointerIterator<ElementType>(Array + N);
		}

		inline constexpr TReversePointerIterator<ElementType> end() const UE_LIFETIMEBOUND
		{
			return TReversePointerIterator<ElementType>(Array);
		}

	private:
		ElementType* Array;
	};
}

/**
 * Allows a range to be iterated backwards.  The container must not be modified during iteration,
 * but elements may be.  The container must outlive the adapter.
 *
 * @param  Range  The range to iterate over backwards.
 *
 * @return  An adapter which allows reverse iteration over the range.  The adapter is not intended
 *          to be handled directly, but rather as a parameter to an algorithm or ranged-for loop.
 *
 * Example:
 *
 * TArray<int> Array = { 1, 2, 3, 4, 5 };
 *
 * for (int& Element : ReverseIterate(Array))
 * {
 *     UE_LOGF(LogTemp, Log, "%d", Element);
 * }
 * // Logs: 5 4 3 2 1
 *
 * TArray<FString> Output;
 * Algo::Transform(ReverseIterate(Array), Output, UE_PROJECTION(LexToString));
 * // Output: [ "5", "4", "3", "2", "1" ]
 */
template <typename RangeType>
inline constexpr UE::Core::Private::TReverseIterationAdapter<RangeType> ReverseIterate(RangeType&& Range UE_LIFETIMEBOUND)
{
	return UE::Core::Private::TReverseIterationAdapter<RangeType>(Range);
}
