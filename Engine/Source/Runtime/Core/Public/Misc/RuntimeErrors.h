// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformMisc.h"
#include "Logging/LogVerbosity.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Build.h"
#include "Misc/CoreDelegates.h"

class FText;

// By default we bubble up runtime errors to the editor or log under the same circumstances as we assert with check/ensure
#if !defined(UE_RAISE_RUNTIME_ERRORS)
#define UE_RAISE_RUNTIME_ERRORS DO_CHECK
#endif

struct FRuntimeErrors
{
	/**
	 * Prints out a runtime warning or error (typically by 'throwing' a BP script exception with the BP callstack)
	 *
	 * @param	Verbosity	The verbosity of the issue (should be Warning or Error)
	 * @param	File	File name ANSI string (__FILE__)
	 * @param	Line	Line number (__LINE__)
	 * @param	Message	Error or warning message to display, this may be surfaced to the user in the editor so make sure it is actionable (e.g., includes asset name or other useful info to help address the problem)
	 *
	 * @return false in all cases.
	 */
	static CORE_API void LogRuntimeIssue(ELogVerbosity::Type Verbosity, const ANSICHAR* FileName, int32 LineNumber, const FText& Message);

	// The style of delegate called when a runtime issue occurs
	DECLARE_DELEGATE_FourParams(FRuntimeErrorDelegate, ELogVerbosity::Type /*Verbosity*/, const ANSICHAR* /*File*/, int32 /*Line*/, const FText& /*Message*/);

	// A delegate this is called when a runtime issue is raised
	static CORE_API FRuntimeErrorDelegate OnRuntimeIssueLogged;

	/**
	 * Raises a runtime error and returns false.
	 *
	 * @param	Expr	Code expression ANSI string (#code)
	 * @param	File	File name ANSI string (__FILE__)
	 * @param	Line	Line number (__LINE__)
	 *
	 * @return false in all cases.
	 */
	static CORE_API bool LogRuntimeIssueReturningFalse(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line);

private:
	FRuntimeErrors() {}
};

#if UE_RAISE_RUNTIME_ERRORS && !USING_CODE_ANALYSIS
	// Raises a runtime error to the log, possibly stopping in the BP debugger, but then returning execution to the caller
	#define LogRuntimeError(Message) FRuntimeErrors::LogRuntimeIssue(ELogVerbosity::Error, __FILE__, __LINE__, Message)

	// Raises a runtime warning to the log, possibly stopping in the BP debugger, but then returning execution to the caller
	#define LogRuntimeWarning(Message) FRuntimeErrors::LogRuntimeIssue(ELogVerbosity::Warning, __FILE__, __LINE__, Message)

	/**
	 * ensureAsRuntimeWarning() can be used to test for *non-fatal* errors at runtime.
	 *
	 * It is a lot like ensure() but it does not generate a C++ callstack or break in the C++ debugger.
	 *
	 * Rather than crashing or stopping in the C++ debugger, an error report with a script callstack will be printed to the output log.
	 * This is useful when you want runtime code verification but you're handling the error case anyway.
	 *
	 * Note: ensureAsRuntimeWarning() can be nested within conditionals!
	 *
	 * Example:
	 *
	 *		if (ensureAsRuntimeWarning(InObject != nullptr))
	 *		{
	 *			InObject->Modify();
	 *		}
	 *
	 * This code is safe to execute as the pointer dereference is wrapped in a non-nullptr conditional block, but
	 * you still want to find out if this ever happens so you can avoid side effects.
	 *
	 * ensureAsRuntimeWarning() resolves to just evaluate the expression when UE_RAISE_RUNTIME_ERRORS is 0 (typically shipping or test builds).
	 */
	#define ensureAsRuntimeWarning(InExpression) (LIKELY(!!(InExpression)) || FRuntimeErrors::LogRuntimeIssueReturningFalse(#InExpression, __FILE__, __LINE__))
#else
	#define LogRuntimeError(Message)
	#define LogRuntimeWarning(Message)
	#define ensureAsRuntimeWarning(InExpression) (!!(InExpression))

#endif

