// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace ScopeExitSupport
{
	/**
	 * Not meant for direct consumption : use ON_SCOPE_EXIT instead.
	 *
	 * RAII class that calls a lambda when it is destroyed.
	 */
	template <typename FuncType>
	class TScopeGuard
	{
		TScopeGuard(TScopeGuard&&) = delete;
		TScopeGuard(const TScopeGuard&) = delete;
		TScopeGuard& operator=(TScopeGuard&&) = delete;
		TScopeGuard& operator=(const TScopeGuard&) = delete;

	public:
		// Given a lambda, constructs an RAII scope guard.
		explicit TScopeGuard(FuncType&& InFunc)
			: Func((FuncType&&)InFunc)
		{
		}

		// Causes the lambda to be executed.
		~TScopeGuard()
		{
			Func();
		}

	private:
		// The lambda to be executed when this guard goes out of scope.
		FuncType Func;
	};

	struct FScopeGuardSyntaxSupport
	{
		template <typename FuncType>
		TScopeGuard<FuncType> operator+(FuncType&& InFunc)
		{
			return TScopeGuard<FuncType>((FuncType&&)InFunc);
		}
	};
}

#define UE_PRIVATE_SCOPE_EXIT_JOIN(A, B) UE_PRIVATE_SCOPE_EXIT_JOIN_INNER(A, B)
#define UE_PRIVATE_SCOPE_EXIT_JOIN_INNER(A, B) A##B



/**
 * Enables a lambda to be executed on scope exit.
 *
 * Example:
 *    {
 *      FileHandle* Handle = GetFileHandle();
 *      ON_SCOPE_EXIT
 *      {
 *          CloseFile(Handle);
 *      };
 *
 *      DoSomethingWithFile( Handle );
 *
 *      // File will be closed automatically no matter how the scope is exited, e.g.:
 *      // * Any return statement.
 *      // * break or continue (if the scope is a loop body).
 *      // * An exception is thrown outside the block.
 *      // * Execution reaches the end of the block.
 *    }
 */
#define ON_SCOPE_EXIT const auto UE_PRIVATE_SCOPE_EXIT_JOIN(ScopeGuard_, __LINE__) = ::ScopeExitSupport::FScopeGuardSyntaxSupport() + [&]() -> void
