// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/NotNull.h"
#include "HAL/CriticalSection.h"

namespace UE
{
	// RAII-style scope locking of a synchronization primitive.
	// `MutexType` is required to implement `Lock` and `Unlock` methods.
	// Example:
	//	{
	//		TScopeLock<FCriticalSection> ScopeLock(CriticalSection);
	//		...
	//	}
	template<typename MutexType>
	class TScopeLock
	{
	public:
		UE_NONCOPYABLE(TScopeLock);

		[[nodiscard]] explicit TScopeLock(MutexType& InMutex) : Mutex(&InMutex)
		{
			check(Mutex);
			Mutex->Lock();
		}

		~TScopeLock()
		{
			Unlock();
		}

		void Unlock()
		{
			if (Mutex)
			{
				Mutex->Unlock();
				Mutex = nullptr;
			}
		}

	private:
		MutexType* Mutex;
	};

	// RAII-style scope locking of a synchronization primitive. Same
	// as TScopeLock except taking the lock is conditional.
	template<typename MutexType>
	class TConditionalScopeLock
	{
	public:
		UE_NONCOPYABLE(TConditionalScopeLock);

		[[nodiscard]] explicit TConditionalScopeLock(MutexType& InMutex, bool bShouldLock)
			: Mutex(bShouldLock ? &InMutex : nullptr)
		{
			if (bShouldLock)
			{
				check(Mutex);
				Mutex->Lock();
			}
		}

		~TConditionalScopeLock()
		{
			Unlock();
		}

		void Unlock()
		{
			if (Mutex)
			{
				Mutex->Unlock();
				Mutex = nullptr;
			}
		}

	private:
		MutexType* Mutex;
	};

	// RAII-style scope unlocking of a synchronization primitive.
	// `MutexType` is required to implement `Lock` and `Unlock` methods.
	// Example:
	//	{
	//		TScopeLock<FCriticalSection> ScopeLock(CriticalSection);
	//		for (FElementType& Element : ThreadUnsafeContainer)
	//		{
	//			TScopeUnlock<FCriticalSection> ScopeUnlock(&CriticalSection);
	//			Process(Element);
	//		}
	//	}
	template<typename MutexType>
	class TScopeUnlock
	{
	public:
		UE_NONCOPYABLE(TScopeUnlock);

		[[nodiscard]] explicit TScopeUnlock(MutexType* InMutex) : Mutex(InMutex)
		{
			if (Mutex)
			{
				Mutex->Unlock();
			}
		}

		~TScopeUnlock()
		{
			if (Mutex)
			{
				Mutex->Lock();
			}
		}

	private:
		MutexType* Mutex;
	};
}

/**
 * Implements a scope lock.
 *
 * This is a utility class that handles scope level locking. It's very useful
 * to keep from causing deadlocks due to exceptions being caught and knowing
 * about the number of locks a given thread has on a resource. Example:
 *
 * <code>
 *	{
 *		// Synchronize thread access to the following data
 *		FScopeLock ScopeLock(SyncObject);
 *		// Access data that is shared among multiple threads
 *		...
 *		// When ScopeLock goes out of scope, other threads can access data
 *	}
 * </code>
 */
class FScopeLock : private UE::TScopeLock<FCriticalSection>
{
public:
	/**
	 * Constructor that performs a lock on the synchronization object
	 *
	 * @param InSyncObject The synchronization object to manage
	 */
	[[nodiscard]] explicit FScopeLock(TNotNull<FCriticalSection*> InSyncObject) : UE::TScopeLock<FCriticalSection>(*InSyncObject)
	{
	}

	using UE::TScopeLock<FCriticalSection>::Unlock;
};

/**
 * Implements a scope unlock.
 *
 * This is a utility class that handles scope level unlocking. It's very useful
 * to allow access to a protected object when you are sure it can happen.
 * Example:
 *
 * <code>
 *	{
 *		// Access data that is shared among multiple threads
 *		FScopeUnlock ScopeUnlock(SyncObject);
 *		...
 *		// When ScopeUnlock goes out of scope, other threads can no longer access data
 *	}
 * </code>
 */
class FScopeUnlock : private UE::TScopeUnlock<FCriticalSection>
{
public:
	/**
	 * Constructor that performs a unlock on the synchronization object
	 *
	 * @param InSyncObject The synchronization object to manage, can be null.
	 */
	[[nodiscard]] explicit FScopeUnlock(FCriticalSection* InSyncObject) : UE::TScopeUnlock<FCriticalSection>(InSyncObject)
	{
	}
};
