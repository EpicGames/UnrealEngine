// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/CriticalSection.h"

namespace UE
{

// RAII-style scope locking of a synchronisation primitive
// `MutexType` is required to implement `ReadLock` and `ReadUnlock` methods
// Example:
//	{
//		TReadScopeLock<FRWLock> ScopeLock(RWLock);
//		...
//	}
template<typename MutexType>
class TReadScopeLock
{
public:
	UE_NONCOPYABLE(TReadScopeLock);

	UE_NODISCARD_CTOR TReadScopeLock(MutexType& InMutex)
		: Mutex(&InMutex)
	{
		check(Mutex);
		Mutex->ReadLock();
	}

	~TReadScopeLock()
	{
		ReadUnlock();
	}

	void ReadUnlock()
	{
		if (Mutex)
		{
			Mutex->ReadUnlock();
			Mutex = nullptr;
		}
	}

private:
	MutexType* Mutex;
};


// RAII-style scope locking of a synchronisation primitive
// `MutexType` is required to implement `WriteLock` and `WriteUnlock` methods
// Example:
//	{
//		TWriteScopeLock<FRWLock> ScopeLock(RWLock);
//		...
//	}
template<typename MutexType>
class TWriteScopeLock
{
public:
	UE_NONCOPYABLE(TWriteScopeLock);

	UE_NODISCARD_CTOR TWriteScopeLock(MutexType& InMutex)
		: Mutex(&InMutex)
	{
		check(Mutex);
		Mutex->WriteLock();
	}

	~TWriteScopeLock()
	{
		WriteUnlock();
	}

	void WriteUnlock()
	{
		if (Mutex)
		{
			Mutex->WriteUnlock();
			Mutex = nullptr;
		}
	}

private:
	MutexType* Mutex;
};

} // namespace UE

/** Keeps a FRWLock read-locked while this scope lives */
class FReadScopeLock
{
public:
	UE_NODISCARD_CTOR explicit FReadScopeLock(FRWLock& InLock)
		: Lock(InLock)
	{
		Lock.ReadLock();
	}

	~FReadScopeLock()
	{
		Lock.ReadUnlock();
	}

private:
	FRWLock& Lock;

	UE_NONCOPYABLE(FReadScopeLock);
};

/** Keeps a FRWLock write-locked while this scope lives */
class FWriteScopeLock
{
public:
	UE_NODISCARD_CTOR explicit FWriteScopeLock(FRWLock& InLock)
		: Lock(InLock)
	{
		Lock.WriteLock();
	}

	~FWriteScopeLock()
	{
		Lock.WriteUnlock();
	}
	
private:
	FRWLock& Lock;

	UE_NONCOPYABLE(FWriteScopeLock);
};

//
// A scope lifetime controlled Read or Write lock of referenced mutex object
//
enum FRWScopeLockType
{
	SLT_ReadOnly = 0,
	SLT_Write,
};

namespace UE 
{

// RAII-style scope locking of a synchronisation primitive
// `MutexType` is required to implement `ReadLock` `WriteLock` `ReadUnlock` and `WriteUnlock` methods
// Example:
//	{
//		TRWScopeLock<FRWLock> ScopeLock(RWLock, SLT_ReadOnly or SLT_Write);
//		...
//	}
template<typename MutexType>
class TRWScopeLock
{
public:
	UE_NONCOPYABLE(TRWScopeLock);

	UE_NODISCARD_CTOR TRWScopeLock(MutexType& InMutex, FRWScopeLockType InLockType)
		: Mutex(&InMutex)
		, LockType(InLockType)
	{
		check(Mutex);
		if (LockType == SLT_ReadOnly)
		{
			Mutex->ReadLock();
		}
		else
		{
			Mutex->WriteLock();
		}
	}

	~TRWScopeLock()
	{
		if (LockType == SLT_ReadOnly)
		{
			Mutex->ReadUnlock();
		}
		else
		{
			Mutex->WriteUnlock();
		}
	}

private:
	MutexType* Mutex;
	FRWScopeLockType LockType;
};

} // namespace UE

/**
 * Keeps a FRWLock read- or write-locked while this scope lives
 *
 *	Notes:
 *		- PThreads and Win32 API's don't provide a mechanism for upgrading a ownership of a read lock to a write lock - to get round that this system unlocks then acquires a write lock so it can race between.
 */
class FRWScopeLock
{
public:
	UE_NODISCARD_CTOR explicit FRWScopeLock(FRWLock& InLockObject,FRWScopeLockType InLockType)
	: LockObject(InLockObject)
	, LockType(InLockType)
	{
		if(LockType != SLT_ReadOnly)
		{
			LockObject.WriteLock();
		}
		else
		{
			LockObject.ReadLock();
		}
	}
	
	// NOTE: As the name suggests, this function should be used with caution. 
	// It releases the read lock _before_ acquiring a new write lock. This is not an atomic operation and the caller should 
	// not treat it as such. 
	// E.g. Pointers read from protected data structures prior to this call may be invalid after the function is called. 
	void ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION()
	{
		if(LockType == SLT_ReadOnly)
		{
			LockObject.ReadUnlock();
			LockObject.WriteLock();
			LockType = SLT_Write;
		}
	}
	
	~FRWScopeLock()
	{
		if(LockType == SLT_ReadOnly)
		{
			LockObject.ReadUnlock();
		}
		else
		{
			LockObject.WriteUnlock();
		}
	}
	
private:
	UE_NONCOPYABLE(FRWScopeLock);

	FRWLock& LockObject;
	FRWScopeLockType LockType;
};
