// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/NotNull.h"
#include "HAL/CriticalSection.h"

namespace UE
{
	/**
	 * RAII-style scope locking of a synchronization primitive using TryLock.
	 * `MutexType` must be a type like `UE::FMutex` or `FCriticalSection`
	 * which implements `TryLock` and `Unlock` methods.
	 * 
	 * <code>
	 *	{
	 *		// Try to acquire a lock on Mutex for the current scope.
	 *		UE::TScopeTryLock ScopeTryLock(Mutex);
	 *		
	 *		// Check that the lock was acquired.
	 *		if (ScopeTryLock.IsLocked())
	 *		{
	 *			// If the lock was acquired, we can safely access resources protected
	 *			// by the mutex.
	 *		}
	 *		
	 *		// When ScopeTryLock goes out of scope, the mutex will be released if it was
	 *		// ever acquired.
	 *	}
	 * </code>
	 */
	template <typename MutexType>
	class TScopeTryLock
	{
	public:
		[[nodiscard]] explicit TScopeTryLock(MutexType& InMutex)
		{
			if (InMutex.TryLock())
			{
				HeldMutex = &InMutex;
			}
		}

		~TScopeTryLock()
		{
			if (HeldMutex)
			{
				HeldMutex->Unlock();
			}
		}

		UE_FORCEINLINE_HINT bool IsLocked()
		{
			return HeldMutex != nullptr;
		}

	private:
		TScopeTryLock(const TScopeTryLock&) = delete;
		TScopeTryLock& operator=(TScopeTryLock&) = delete;

		MutexType* HeldMutex = nullptr;
	};
}

/**
 * Implements a scope lock using TryLock.
 *
 * This is a utility class that handles scope level locking using TryLock.
 * Scope locking helps to avoid programming errors by which a lock is acquired
 * and never released.
 *
 * <code>
 *	{
 *		// Try to acquire a lock on CriticalSection for the current scope.
 *		FScopeTryLock ScopeTryLock(CriticalSection);
 *		// Check that the lock was acquired.
 *		if (ScopeTryLock.IsLocked())
 *		{
 *			// If the lock was acquired, we can safely access resources protected
 *			// by the lock.
 *		}
 *		// When ScopeTryLock goes out of scope, the critical section will be
 *		// released if it was ever acquired.
 *	}
 * </code>
 */
class FScopeTryLock : private UE::TScopeTryLock<FCriticalSection>
{
public:
	// Constructor that tries to lock the critical section.
	[[nodiscard]] explicit FScopeTryLock(TNotNull<FCriticalSection*> InCriticalSection) : UE::TScopeTryLock<FCriticalSection>(*InCriticalSection)
	{
	}

	// You must call IsLocked to test that the lock was acquired.
	using UE::TScopeTryLock<FCriticalSection>::IsLocked;
};