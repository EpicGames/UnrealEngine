// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "HAL/IConsoleManager.h"
#include "UObject/NameTypes.h"

/**
* FScopedCVar Allows a CVar to be temporarily set to a particular value.
* Nested scopes still get popped off correctly since ECVF_SetByTemp is
* an array type.
*/

template<typename T>
class FScopedCVar
{
public:
	FScopedCVar(FString InConsoleVariableName, T Value, FName Tag = "FScopedCVar")
		: CachedVariableName(InConsoleVariableName)
		, CachedTag(Tag)
	{
		IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*CachedVariableName);
		if (ensure(ConsoleVariable))
		{
			ConsoleVariable->Set(Value, ECVF_SetByTemp, Tag);
		}
	}

	~FScopedCVar()
	{
		IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*CachedVariableName);
		if (ensure(ConsoleVariable))
		{
			ConsoleVariable->Unset(ECVF_SetByTemp, CachedTag);
		}
	}

	FScopedCVar(const FScopedCVar&) = delete;
	FScopedCVar& operator=(const FScopedCVar&) = delete;

private:
	FString CachedVariableName;
	FName CachedTag;
};
