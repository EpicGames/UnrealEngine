// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/Event.h"

/**
 * This class is allows a simple one-shot scoped event.
 *
 * Usage:
 * {
 *		FScopedEvent MyEvent;
 *		SendReferenceOrPointerToSomeOtherThread(&MyEvent); // Other thread calls MyEvent->Trigger();
 *		// MyEvent destructor is here, we wait here.
 * }
 */
class FScopedEvent
{
public:

	/** Default constructor. */
	CORE_API FScopedEvent();

	/** Destructor. */
	CORE_API ~FScopedEvent();

	/** Triggers the event. */
	void Trigger()
	{
		Event->Trigger();
	}

	/**
	 * Checks if the event has been triggered (used for special early out cases of scope event)
	 * if this returns true once it will return true forever
	 *
	 * @return returns true if the scoped event has been triggered once
	 */
	CORE_API bool IsReady();

	/**
	 * Retrieve the event, usually for passing around.
	 *
	 * @return The event.
	 */
	FEvent* Get()
	{
		return Event;
	}

private:

	/** Holds the event. */
	FEvent* Event;
};
