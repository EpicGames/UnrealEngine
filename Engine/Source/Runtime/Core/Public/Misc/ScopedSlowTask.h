// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Misc/SlowTask.h"

/**
 * A scope block representing an amount of work divided up into sections.
 * Use one scope at the top of each function to give accurate feedback to the user of a slow operation's progress.
 *
 * Example Usage:
 *	void DoSlowWork()
 *	{
 *		FScopedSlowTask Progress(2.f, LOCTEXT("DoingSlowWork", "Doing Slow Work..."));
 *		// Optionally make this show a dialog if not already shown
 *		Progress.MakeDialog();
 *
 *		// Indicate that we are entering a frame representing 1 unit of work
 *		Progress.EnterProgressFrame(1.f);
 *		
 *		// DoFirstThing() can follow a similar pattern of creating a scope divided into frames. These contribute to their parent's progress frame proportionately.
 *		DoFirstThing();
 *		
 *		Progress.EnterProgressFrame(1.f);
 *		DoSecondThing();
 *	}
 *
 */
struct FScopedSlowTask : FSlowTask
{

	/**
	 * Construct this scope from an amount of work to do, and a message to display
	 * @param		InAmountOfWork			Arbitrary number of work units to perform (can be a percentage or number of steps).
	 *										0 indicates that no progress frames are to be entered in this scope (automatically enters a frame encompassing the entire scope)
	 * @param		InDefaultMessage		A message to display to the user to describe the purpose of the scope
	 * @param		bInEnabled				When false, this scope will have no effect. Allows for proper scoped objects that are conditionally disabled.
	 */
	inline FScopedSlowTask(float InAmountOfWork, const FText& InDefaultMessage = FText(), bool bInEnabled = true, FFeedbackContext& InContext = *GWarn)
		: FSlowTask(InAmountOfWork, InDefaultMessage, bInEnabled, InContext)
	{
		Initialize();
	}

	UE_FORCEINLINE_HINT ~FScopedSlowTask()
	{
		Destroy();
	}
};
