// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringConv.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PreprocessorHelpers.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Guid.h"
#include "Serialization/Archive.h"
#include "Serialization/BufferReader.h"
#include "Serialization/MemoryLayout.h"
#include "Stats/Stats.h"
#include "String/BytesToHex.h"
#include "String/HexToBytes.h"
#include "Templates/UnrealTemplate.h"

class FCbFieldView;
class FCbWriter;
class FMemoryImageWriter;
class FMemoryUnfreezeContent;
class FPointerTableBase;
class FSHA1;
struct FMD5Hash;

/*-----------------------------------------------------------------------------
	MD5 functions.
-----------------------------------------------------------------------------*/

/** @name MD5 functions */
//@{
//
// MD5 Context.
//


//
// MD5 functions.
//!!it would be cool if these were implemented as subclasses of
// FArchive.
//
// CORE_API void appMD5Init( FMD5Context* context );
// CORE_API void appMD5Update( FMD5Context* context, uint8* input, int32 inputLen );
// CORE_API void appMD5Final( uint8* digest, FMD5Context* context );
// CORE_API void appMD5Transform( uint32* state, uint8* block );
// CORE_API void appMD5Encode( uint8* output, uint32* input, int32 len );
// CORE_API void appMD5Decode( uint32* output, uint8* input, int32 len );

class FMD5
{
public:
	CORE_API FMD5();
	CORE_API ~FMD5();

	/**
	 * MD5 block update operation.  Continues an MD5 message-digest operation,
	 * processing another message block, and updating the context.
	 *
	 * @param input		input data
	 * @param inputLen	length of the input data in bytes
	 **/
	CORE_API void Update(const uint8* input, uint64 inputLen);

	/**
	 * MD5 finalization. Ends an MD5 message-digest operation, writing the
	 * the message digest and zeroizing the context.
	 * Digest is 16 BYTEs.
	 *
	 * @param digest	pointer to a buffer where the digest should be stored ( must have at least 16 bytes )
	 **/
	CORE_API void Final(uint8* digest);

	/**
	 * Helper to perform the very common case of hashing an ASCII string into a hex representation.
	 * 
	 * @param String	hex representation of the hash (32 lower-case hex digits)
	 **/
	static FString HashAnsiString(const TCHAR* String)
	{
		return HashBytes((unsigned char*)TCHAR_TO_ANSI(String), FCString::Strlen(String));
	}

	/**
	 * Helper to perform the very common case of hashing an in-memory array of bytes into a hex representation
	 *
	 * @param String	hex representation of the hash (32 lower-case hex digits)
	 **/
	static FString HashBytes(const uint8* input, uint64 inputLen)
	{
		uint8 Digest[16];

		FMD5 Md5Gen;

		Md5Gen.Update(input, inputLen);
		Md5Gen.Final(Digest);

		FString MD5;
		for (int32 i = 0; i < 16; i++)
		{
			MD5 += FString::Printf(TEXT("%02x"), Digest[i]);
		}
		return MD5;
	}

private:
	struct FContext
	{
		uint32 state[4];
		uint32 count[2];
		uint8 buffer[64];
	};

	CORE_API void Transform( uint32* state, const uint8* block );
	CORE_API void Encode( uint8* output, const uint32* input, int32 len );
	CORE_API void Decode( uint32* output, const uint8* input, int32 len );

	FContext Context;
};
//@}

struct FMD5Hash;

/** Simple helper struct to ease the caching of MD5 hashes */
struct FMD5Hash
{
	/** Default constructor */
	FMD5Hash() : bIsValid(false) {}

	/** Check whether this has hash is valid or not */
	bool IsValid() const { return bIsValid; }

	/** Set up the MD5 hash from a container */
	void Set(FMD5& MD5)
	{
		MD5.Final(Bytes);
		bIsValid = true;
	}

	/** Compare one hash with another */
	friend bool operator==(const FMD5Hash& LHS, const FMD5Hash& RHS)
	{
		return LHS.bIsValid == RHS.bIsValid && (!LHS.bIsValid || FMemory::Memcmp(LHS.Bytes, RHS.Bytes, 16) == 0);
	}

	/** Compare one hash with another */
	friend bool operator!=(const FMD5Hash& LHS, const FMD5Hash& RHS)
	{
		return LHS.bIsValid != RHS.bIsValid || (LHS.bIsValid && FMemory::Memcmp(LHS.Bytes, RHS.Bytes, 16) != 0);
	}

	/** Serialise this hash */
	friend FArchive& operator<<(FArchive& Ar, FMD5Hash& Hash)
	{
		Ar << Hash.bIsValid;
		if (Hash.bIsValid)
		{
			Ar.Serialize(Hash.Bytes, 16);
		}

		return Ar;
	}

	/** Hash the specified file contents (using the optionally supplied scratch buffer) */
	CORE_API static FMD5Hash HashFile(const TCHAR* InFilename, TArray<uint8>* Buffer = nullptr);
	CORE_API static FMD5Hash HashFileFromArchive(FArchive* Ar, TArray<uint8>* ScratchPad = nullptr);

	const uint8* GetBytes() const { return Bytes; }
	const int32 GetSize() const { return sizeof(Bytes); }

private:
	/** Whether this hash is valid or not */
	bool bIsValid;

	/** The bytes this hash comprises */
	uint8 Bytes[16];

	friend inline FCbWriter& operator<<(FCbWriter& Writer, const FMD5Hash& Hash) // Hidden friends must be inlined to be performant
	{
		return Hash.WriteCompactBinary(Writer);
	}
	friend bool LoadFromCompactBinary(const FCbFieldView& Field, FMD5Hash& OutHash) { return OutHash.LoadFromCompactBinary(Field); }
	friend FString LexToString(const FMD5Hash& InHash) { return InHash.LexToString(); }
	friend void LexFromString(FMD5Hash& Hash, const TCHAR* InString) { Hash.LexFromString(InString); }

	CORE_API FCbWriter& WriteCompactBinary(FCbWriter& Writer) const;
	CORE_API bool LoadFromCompactBinary(const FCbFieldView& Field); // inlining not available because we don't want FCbFieldView defined
	CORE_API FString LexToString() const;
	CORE_API void LexFromString(const TCHAR*);
};

/** 
  * Construct a FGuid from a MD5Hash. This means that calling ToString on the resulting FGuid will not result in the 
  * expected MD5 hash string, due to how FGuid outputs the string; LexToString should be used in that case.
  */
inline FGuid MD5HashToGuid(const FMD5Hash& Hash)
{
	FGuid Result;
	FMemory::Memcpy(&Result, Hash.GetBytes(), sizeof(FGuid));
	return Result;
}

/*-----------------------------------------------------------------------------
	SHA-1 functions.
-----------------------------------------------------------------------------*/

/*
 *	NOTE:
 *	100% free public domain implementation of the SHA-1 algorithm
 *	by Dominik Reichl <dominik.reichl@t-online.de>
 *	Web: http://www.dominik-reichl.de/
 */

/** This divider string is beween full file hashes and script hashes */
#define HASHES_SHA_DIVIDER "+++"

/** Stores an SHA hash generated by FSHA1. */
class FSHAHash
{
public:
	alignas(uint32) uint8 Hash[20];

	FSHAHash()
	{
		FMemory::Memset(Hash, 0, sizeof(Hash));
	}

	static constexpr int32 GetStringLen() { return UE_ARRAY_COUNT(Hash) * 2; }

	inline void AppendString(FString& Out) const
	{
		BytesToHex((const uint8*)Hash, sizeof(Hash), Out);
	}

	inline FString ToString() const
	{
		return BytesToHex((const uint8*)Hash, sizeof(Hash));
	}

	inline void ToString(TCHAR* Dest, bool bNullTerminate) const
	{
		constexpr auto Count = UE_ARRAY_COUNT(Hash);
		for (int i = 0; i < Count; ++i)
		{
			uint8 Val = Hash[i];
			Dest[i * 2] = NibbleToTChar(Val >> 4);
			Dest[i * 2 + 1] = NibbleToTChar(Val & 15);
		}

		if (bNullTerminate)
		{
			Dest[Count * 2] = TEXT('\0');
		}
	}

	inline void FromString(const FStringView& Src)
	{
		check(Src.Len() == GetStringLen());
		UE::String::HexToBytes(Src, Hash);
	}

	friend bool operator==(const FSHAHash& X, const FSHAHash& Y)
	{
		return FMemory::Memcmp(&X.Hash, &Y.Hash, sizeof(X.Hash)) == 0;
	}

	friend bool operator!=(const FSHAHash& X, const FSHAHash& Y)
	{
		return FMemory::Memcmp(&X.Hash, &Y.Hash, sizeof(X.Hash)) != 0;
	}

	friend bool operator<(const FSHAHash& X, const FSHAHash& Y)
	{
		return FMemory::Memcmp(&X.Hash, &Y.Hash, sizeof(X.Hash)) < 0;
	}

	friend FArchive& operator<<( FArchive& Ar, FSHAHash& G )
	{
		Ar.Serialize(&G.Hash, sizeof(G.Hash));
		return Ar;
	}
	
	friend uint32 GetTypeHash(const FSHAHash& InKey)
	{
		return *reinterpret_cast<const uint32*>(InKey.Hash);
	}

	friend FString LexToString(const FSHAHash& InHash) { return InHash.ToString(); }
	friend void LexFromString(FSHAHash& InHash, const TCHAR* InString) { InHash.FromString(InString); }
	friend FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FSHAHash& InHash) { UE::String::BytesToHex(InHash.Hash, Builder); return Builder; }
	friend FAnsiStringBuilderBase& operator<<(FAnsiStringBuilderBase& Builder, const FSHAHash& InHash) { UE::String::BytesToHex(InHash.Hash, Builder); return Builder; }
	friend FCbWriter& operator<<(FCbWriter& Writer, const FSHAHash& InHash) { return InHash.WriteCompactBinary(Writer); }
	friend bool LoadFromCompactBinary(const FCbFieldView& Field, FSHAHash& OutHash) { return OutHash.LoadFromCompactBinary(Field); }

	CORE_API bool LoadFromCompactBinary(const FCbFieldView& Field);
	CORE_API FCbWriter& WriteCompactBinary(FCbWriter& Writer) const;
};

namespace Freeze
{
	CORE_API void IntrinsicToString(const FSHAHash& Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
}

DECLARE_INTRINSIC_TYPE_LAYOUT(FSHAHash);

class FSHA1
{
public:

	enum {DigestSize=20};
	// Constructor and Destructor
	CORE_API FSHA1();
	CORE_API ~FSHA1();

	uint32 m_state[5];
	uint64 m_count;
	uint32 __reserved1[1];
	uint8  m_buffer[64];
	uint8  m_digest[20];
	uint32 __reserved2[3];

	CORE_API void Reset();

	// Update the hash value
	CORE_API void Update(const uint8 *data, uint64 len);

	// Update the hash value with string
	CORE_API void UpdateWithString(const TCHAR *data, uint32 len);

	// Update with POD data.
	template <typename Type	UE_REQUIRES(TIsPODType<Type>::Value)>
	void Update(const Type& InData)
	{
		Update(reinterpret_cast<const uint8*>(&InData), sizeof(Type));
	}

	// Finalize hash and report
	CORE_API void Final();

	// Finalize hash and return it
	FSHAHash Finalize()
	{
		Final();
		FSHAHash Digest;
		GetHash(reinterpret_cast<uint8*>(&Digest));
		return Digest;
	}

	// Report functions: as pre-formatted and raw data
	CORE_API void GetHash(uint8 *puDest) const;

	/**
	 * Calculate the hash on a single block and return it
	 *
	 * @param Data Input data to hash
	 * @param DataSize Size of the Data block
	 * @param OutHash Resulting hash value (20 byte buffer)
	 */
	static CORE_API void HashBuffer(const void* Data, uint64 DataSize, uint8* OutHash);

	/**
	 * Calculate the hash on a single block and return it
	 *
	 * @param Data Input data to hash
	 * @param DataSize Size of the Data block
	 * @return Resulting digest
	 */
	static FSHAHash HashBuffer(const void* Data, uint64 DataSize)
	{
		FSHAHash Hash;
		HashBuffer(Data, DataSize, Hash.Hash);
		return Hash;
	}

	/**
	 * Generate the HMAC (Hash-based Message Authentication Code) for a block of data.
	 * https://en.wikipedia.org/wiki/Hash-based_message_authentication_code
	 *
	 * @param Key		The secret key to be used when generating the HMAC
	 * @param KeySize	The size of the key
	 * @param Data		Input data to hash
	 * @param DataSize	Size of the Data block
	 * @param OutHash	Resulting hash value (20 byte buffer)
	 */
	static CORE_API void HMACBuffer(const void* Key, uint32 KeySize, const void* Data, uint64 DataSize, uint8* OutHash);

	/**
	 * Shared hashes.sha reading code (each platform gets a buffer to the data,
	 * then passes it to this function for processing)
	 *
	 * @param Buffer Contents of hashes.sha (probably loaded from an a section in the executable)
	 * @param BufferSize Size of Buffer
	 * @param bDuplicateKeyMemory If Buffer is not always loaded, pass true so that the 20 byte hashes are duplicated 
	 */
	static CORE_API void InitializeFileHashesFromBuffer(uint8* Buffer, uint64 BufferSize, bool bDuplicateKeyMemory=false);

	/**
	 * Gets the stored SHA hash from the platform, if it exists. This function
	 * must be able to be called from any thread.
	 *
	 * @param Pathname Pathname to the file to get the SHA for
	 * @param Hash 20 byte array that receives the hash
	 * @param bIsFullPackageHash true if we are looking for a full package hash, instead of a script code only hash
	 *
	 * @return true if the hash was found, false otherwise
	 */
	static CORE_API bool GetFileSHAHash(const TCHAR* Pathname, uint8 Hash[20], bool bIsFullPackageHash=true);

private:
	// Private SHA-1 transformation
	CORE_API void Transform(const uint8* buffer, uint64 len);

	/** Global map of filename to hash value, filled out in InitializeFileHashesFromBuffer */
	static CORE_API TMap<FString, uint8*> FullFileSHAHashMap;

	/** Global map of filename to hash value, but for script-only SHA hashes */
	static CORE_API TMap<FString, uint8*> ScriptSHAHashMap;
};


/**
 * Asynchronous SHA verification
 */
class FAsyncSHAVerify
{
protected:
	/** Buffer to run the has on. This class can take ownership of the buffer is bShouldDeleteBuffer is true */
	void* Buffer;

	/** Size of Buffer */
	uint64 BufferSize;

	/** Hash to compare against */
	uint8 Hash[20];

	/** Filename to lookup hash value (can be empty if hash was passed to constructor) */
	FString Pathname;

	/** If this is true, and looking up the hash by filename fails, this will abort execution */
	bool bIsUnfoundHashAnError;

	/** Should this class delete the buffer memory when verification is complete? */
	bool bShouldDeleteBuffer;

public:

	/**
	 * Constructor. 
	 * 
	 * @param	InBuffer				Buffer of data to calculate has on. MUST be valid until this task completes (use Counter or pass ownership via bInShouldDeleteBuffer)
	 * @param	InBufferSize			Size of InBuffer
	 * @param	bInShouldDeleteBuffer	true if this task should FMemory::Free InBuffer on completion of the verification. Useful for a fire & forget verification
	 *									NOTE: If you pass ownership to the task MAKE SURE you are done using the buffer as it could go away at ANY TIME
	 * @param	Pathname				Pathname to use to have the platform lookup the hash value
	 * @param	bInIsUnfoundHashAnError true if failing to lookup the hash value results in a fail (only for Shipping PC)
	 */
	FAsyncSHAVerify(
		void* InBuffer, 
		uint64 InBufferSize, 
		bool bInShouldDeleteBuffer, 
		const TCHAR* InPathname, 
		bool bInIsUnfoundHashAnError)
		:	Buffer(InBuffer)
		,	BufferSize(InBufferSize)
		,	Pathname(InPathname)
		,	bIsUnfoundHashAnError(bInIsUnfoundHashAnError)
		,	bShouldDeleteBuffer(bInShouldDeleteBuffer)
	{
	}

	/**
	 * Performs the async hash verification
	 */
	CORE_API void DoWork();

	/**
	 * Task API, return true to indicate that we can abandon
	 */
	bool CanAbandon()
	{
		return true;
	}

	/**
	 * Abandon task, deletes the buffer if that is what was requested
	 */
	void Abandon()
	{
		if( bShouldDeleteBuffer )
		{
			FMemory::Free( Buffer );
			Buffer = 0;
		}
	}

	UE_FORCEINLINE_HINT TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncSHAVerify, STATGROUP_ThreadPoolAsyncTasks);
	}
};

/**
 * Callback that is called if the asynchronous SHA verification fails
 * This will be called from a pooled thread.
 *
 * NOTE: Each platform is expected to implement this!
 *
 * @param FailedPathname Pathname of file that failed to verify
 * @param bFailedDueToMissingHash true if the reason for the failure was that the hash was missing, and that was set as being an error condition
 */
CORE_API void appOnFailSHAVerification(const TCHAR* FailedPathname, bool bFailedDueToMissingHash);

/**
 * Similar to FBufferReader, but will verify the contents of the buffer on close (on close to that 
 * we know we don't need the data anymore)
 */
class FBufferReaderWithSHA final : public FBufferReaderBase
{
public:
	/**
	 * Constructor
	 * 
	 * @param Data Buffer to use as the source data to read from
	 * @param Size Size of Data
	 * @param bInFreeOnClose If true, Data will be FMemory::Free'd when this archive is closed
	 * @param SHASourcePathname Path to the file to use to lookup the SHA hash value
	 * @param bIsPersistent Uses this value for SetIsPersistent()
	 * @param bInIsUnfoundHashAnError true if failing to lookup the hash should trigger an error (only in ShippingPC)
	 */
	FBufferReaderWithSHA( 
		void* Data, 
		int64 Size, 
		bool bInFreeOnClose, 
		const TCHAR* SHASourcePathname, 
		bool bIsPersistent=false, 
		bool bInIsUnfoundHashAnError=false 
		)
	// we force the base class to NOT free buffer on close, as we will let the SHA task do it if needed
	: FBufferReaderBase(Data, Size, bInFreeOnClose, bIsPersistent)
	, SourcePathname(SHASourcePathname)
	, bIsUnfoundHashAnError(bInIsUnfoundHashAnError)
	{
	}

	CORE_API virtual ~FBufferReaderWithSHA();

	CORE_API bool Close() override;

	/**
  	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const { return TEXT("FBufferReaderWithSHA"); }

protected:
	/** Path to the file to use to lookup the SHA hash value */
	FString SourcePathname;
	/** true if failing to lookup the hash should trigger an error */
	bool bIsUnfoundHashAnError;
};



