// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/Event.h"

/**
 * Fake event object used when running with only one thread.
 */
class FSingleThreadEvent
	: public FEvent
{
	/** Flag to know whether this event has been triggered. */
	bool bTriggered;

	/** Should this event reset automatically or not. */
	bool bManualReset;

public:

	/** Default constructor. */
	FSingleThreadEvent()
		: bTriggered(false)
		, bManualReset(false)
	{ }

public:

	// FEvent Interface

	virtual bool Create( bool bIsManualReset = false ) override
	{ 
		bManualReset = bIsManualReset;
		return true; 
	}

	virtual bool IsManualReset() override
	{
		return bManualReset;
	}

	virtual void Trigger() override
	{
		bTriggered = true;
	}

	virtual void Reset() override
	{
		bTriggered = false;
	}

	virtual bool Wait( uint32 WaitTime, const bool bIgnoreThreadIdleStats = false ) override
	{ 
		// With only one thread it's assumed the event has been triggered
		// before Wait is called, otherwise it would end up waiting forever or always fail.
		check(bTriggered);
		bTriggered = bManualReset;
		return true; 
	}
};
