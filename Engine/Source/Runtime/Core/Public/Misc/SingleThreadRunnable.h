// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/** 
 * Interface for ticking runnables when there's only one thread available and 
 * multithreading is disabled.
 */
class FSingleThreadRunnable
{
public:

	virtual ~FSingleThreadRunnable() = default;

	/* Tick function. */
	virtual void Tick() = 0;
};
