// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"

class FFeedbackContext;

/** Enum to specify a particular slow task section should be shown */
enum class ESlowTaskVisibility
{
	/** Default visibility (inferred by some heuristic of remaining work/time open) */
	Default,
	/** Ignore the required minimum visible time and non-empty text requirement */
	ForceVisible,
	/** Force this particular slow task to be visible on the UI, overriding tasks if there is no space in ascending order of recency (newest tasks at the bottom first) */
	Important,
	/** Forcibly prevent this slow task from being shown, but still use it for work progress calculations */
	Invisible,
};

/**
 * Data type used to store information about a currently running slow task. Direct use is not advised, use FScopedSlowTask instead
 */
struct FSlowTask
{
	/** Default message to display to the user when not overridden by a frame */
	FText DefaultMessage;

	/** Message pertaining to the current frame's work */
	FText FrameMessage;

	/** The amount of work to do in this scope */
	float TotalAmountOfWork;

	/** The amount of work we have already completed in this scope */
	float CompletedWork;

	/** The amount of work the current frame is responsible for */
	float CurrentFrameScope;

	/** The visibility of this slow task */
	ESlowTaskVisibility Visibility;

	/** The time that this scope was created */
	double StartTime;

	/** Threshold before dialog is opened */
	TOptional<float> OpenDialogThreshold;

private:

	/** Boolean flag to control whether this scope actually does anything (unset for quiet operations etc) */
	bool bEnabled;

	/** Flag that specifies whether this feedback scope created a new slow task dialog */
	bool bCreatedDialog;
	
	/** The feedback context that we belong to */
	FFeedbackContext& Context;

	/** Specify whether the delayed dialog should show a cancel button */
	bool bDelayedDialogShowCancelButton : 1;

	/** Specify whether the delayed dialog is allowed in PIE */
	bool bDelayedDialogAllowInPIE : 1;

	/** Avoid calling hierarchical dialog creation if not required */
	bool bSkipRecursiveDialogCreation : 1;

	/** Id of the traced timing region */
	uint64 RegionId;

	/** Prevent copying */
	FSlowTask(const FSlowTask&);

	/** Whenever we encounter edge condition requiring to force a UI refresh */
	static void ForceRefresh(FFeedbackContext& Context);

	/** Handle dialog and delayed dialog creation logic */
	bool MakeDialogIfNeeded();

	/** Call MakeDialogIfNeeded on all scopes recursively to give them a chance to create a delayed dialog in the hierarchy */
	void MakeRecursiveDialogIfNeeded();

public:

	/**
	 * True if a slow task should be created. False if a slow task should not be created because another was created recently.
	 */
	static CORE_API bool ShouldCreateThrottledSlowTask();

	/**
	 * Construct this scope from an amount of work to do, and a message to display
	 * @param		InAmountOfWork			Arbitrary number of work units to perform (can be a percentage or number of steps).
	 *										0 indicates that no progress frames are to be entered in this scope (automatically enters a frame encompassing the entire scope)
	 * @param		InDefaultMessage		A message to display to the user to describe the purpose of the scope
	 * @param		bInVisible				When false, this scope will have no effect. Allows for proper scoped objects that are conditionally hidden.
	 */
	CORE_API FSlowTask(float InAmountOfWork, const FText& InDefaultMessage = FText(), bool bInEnabled = true, FFeedbackContext& InContext = *GWarn);

	/** Function that initializes the scope by adding it to its context's stack */
	CORE_API void Initialize();

	/** Function that finishes any remaining work and removes itself from the global scope stack */
	CORE_API void Destroy();

	/**
	 * Creates a new dialog for this slow task after the given time threshold. If the task completes before this time, no dialog will be shown.
	 * @param		Threshold				Time in seconds before dialog will be shown.
	 * @param		bShowCancelButton		Whether to show a cancel button on the dialog or not
	 * @param		bAllowInPIE				Whether to allow this dialog in PIE. If false, this dialog will not appear during PIE sessions.
	 */
	CORE_API void MakeDialogDelayed(float Threshold, bool bShowCancelButton = false, bool bAllowInPIE = false);

	/**
	 * Creates a new dialog for this slow task, if there is currently not one open
	 * @param		bShowCancelButton		Whether to show a cancel button on the dialog or not
	 * @param		bAllowInPIE				Whether to allow this dialog in PIE. If false, this dialog will not appear during PIE sessions.
	 */
	CORE_API void MakeDialog(bool bShowCancelButton = false, bool bAllowInPIE = false);

	/**
	 * Indicate that we are to enter a frame that will take up the specified amount of work. Completes any previous frames (potentially contributing to parent scopes' progress).
	 * @param		ExpectedWorkThisFrame	The amount of work that will happen between now and the next frame, as a numerator of TotalAmountOfWork.
	 * @param		Text					Optional text to describe this frame's purpose.
	 */
	CORE_API void EnterProgressFrame(float ExpectedWorkThisFrame = 1.f, const FText& Text = FText());

	/**
	 * Let the UI refresh but doesn't advance progress. This should be called at regular intervals even when no progress is being made to keep the UI responsive.
	 */
	CORE_API void TickProgress();

	/** 
	* Whenever we encounter edge condition requiring to force a UI refresh
	* (Like if another popup would show when the slow task finishes and you want it to show 100%)
	*/
	CORE_API void ForceRefresh();

	/**
	 * Get the frame message or default message if empty
	 */
	CORE_API const FText& GetCurrentMessage() const;

	/**
	 * True if the user has requested that the slow task be canceled
	 */
	CORE_API bool ShouldCancel() const;
};
