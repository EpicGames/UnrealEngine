// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"

struct FSlowTask;

/** A stack of feedback scopes */
struct FSlowTaskStack : TArray<FSlowTask*>
{
	/** Get the fraction of work that has been completed for the specified index in the stack (0=total progress) */
	CORE_API float GetProgressFraction(int32 Index) const;
};
