// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"

#ifndef UE_INCLUDE_SOURCE_LOCATION
#define UE_INCLUDE_SOURCE_LOCATION !UE_BUILD_SHIPPING
#endif

#if UE_INCLUDE_SOURCE_LOCATION && defined(__cpp_lib_source_location)
#include <source_location>
#endif

namespace UE
{
	class FSourceLocation
	{
#if UE_INCLUDE_SOURCE_LOCATION
	#ifdef __cpp_lib_source_location
		using FSourceLocationImpl = std::source_location;
	#else // __cpp_lib_source_location
		struct FSourceLocationImpl
		{
			static UE_CONSTEVAL FSourceLocationImpl current(
				const uint_least32_t InLine = __builtin_LINE(),
				const uint_least32_t InColumn = __builtin_COLUMN(),
				const char* const InFile = __builtin_FILE(),
				const char* const InFunction = __builtin_FUNCTION()) noexcept
			{
				FSourceLocationImpl Location;
				Location.Line = InLine;
				Location.Column = InColumn;
				Location.FileName = InFile;
				Location.FunctionName = InFunction;
				return Location;
			};

			constexpr FSourceLocationImpl() noexcept = default;

			// source location field access
			constexpr uint_least32_t line() const noexcept
			{
				return Line;
			}
			constexpr uint_least32_t column() const noexcept
			{
				return Column;
			}
			constexpr const char* file_name() const noexcept
			{
				return FileName;
			}
			constexpr const char* function_name() const noexcept
			{
				return FunctionName;
			}

		private:
			uint_least32_t Line = 0;
			uint_least32_t Column = 0;
			const char* FileName = nullptr;
			const char* FunctionName = nullptr;
		};
	#endif // __cpp_lib_source_location
#else // UE_INCLUDE_SOURCE_LOCATION
		struct FSourceLocationImpl{
			static UE_CONSTEVAL FSourceLocationImpl current() noexcept
			{
				return {};
			}
			constexpr FSourceLocationImpl() noexcept = default;
			constexpr uint_least32_t line() const noexcept
			{
				return 0;
			}
			constexpr uint_least32_t column() const noexcept
			{
				return 0;
			}
			constexpr const char* file_name() const noexcept
			{
				return "";
			}
			constexpr const char* function_name() const noexcept
			{
				return "";
			}
		};
#endif // UE_INCLUDE_SOURCE_LOCATION

	public:
		/**
		 * Saves current source file location into a RAII container that can be used to log/save the information about the caller
		 * Usage:
		 * 
		 * #include "SourceLocation.h"
		 * #include "SourceLocationUtils.h"
		 * 
		 * void MyFunction(int Param1, int Param2, UE::FSourceLocation Location = UE::FSourceLocation::Current()) {
		 *    UE_LOG(TEXT("My caller is %s"), *UE::SourceLocation::Full(Location).ToString());
		 * }
		 */
		static UE_CONSTEVAL FSourceLocation Current(FSourceLocationImpl Impl = FSourceLocationImpl::current()) noexcept
		{
			return FSourceLocation(MoveTemp(Impl));
		}

		// source location field access
		constexpr uint32 GetLine() const noexcept
		{
			return Impl.line();
		}
		constexpr uint32 GetColumn() const noexcept
		{
			return Impl.column();
		}
		constexpr const char* GetFileName() const noexcept
		{
			return Impl.file_name();
		}
		constexpr const char* GetFunctionName() const noexcept
		{
			return Impl.function_name();
		}

	private:
		constexpr FSourceLocation(FSourceLocationImpl&& In) noexcept
			: Impl(MoveTemp(In))
		{
		}

		const FSourceLocationImpl Impl;
	};

}