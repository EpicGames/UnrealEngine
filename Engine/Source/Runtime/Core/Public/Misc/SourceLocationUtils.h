// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "Misc/SourceLocation.h"
#include "Misc/StringBuilder.h"

class FCbWriter;

namespace UE::SourceLocation::Private
{

	CORE_API void SerializeForLogFull(FCbWriter& Writer, const FSourceLocation& Location);
	CORE_API void SerializeForLogFileAndLine(FCbWriter& Writer, const FSourceLocation& Location);

}  // namespace UE::SourceLocation::Private

namespace UE::SourceLocation
{

	/**
	 * Adapter for formatting a source location with full information.
	 */
	struct FFullAdapter
	{
		const FSourceLocation& Location;

		/**
		 * Converts the source location to a readable format with full information.
		 *
		 * @return String representation of the source location.
		 */
		[[nodiscard]] CORE_API FString ToString() const;
		[[nodiscard]] CORE_API FUtf8String ToUtf8String() const;

		/**
		 * Converts the source location to a readable format with full information, in place.
		 *
		 * @param Out String to fill with the string representation of the source location.
		 */
		CORE_API void ToString(FWideString& Out) const;
		CORE_API void ToString(FUtf8String& Out) const;

		/**
		 * Converts the source location to a readable format with full information, appending to an existing string.
		 *
		 * @param Out StringBuilder to append with the string representation of the source location.
		 */
		CORE_API void AppendString(FWideStringBuilderBase& Out) const;
		CORE_API void AppendString(FUtf8StringBuilderBase& Out) const;

		friend void SerializeForLog(FCbWriter& Writer, const FFullAdapter& Adapter)
		{
			Private::SerializeForLogFull(Writer, Adapter.Location);
		}

		friend FWideStringBuilderBase& operator<<(FWideStringBuilderBase& Builder, const FFullAdapter& Adapter)
		{
			Adapter.AppendString(Builder);
			return Builder;
		}
		friend FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Builder, const FFullAdapter& Adapter)
		{
			Adapter.AppendString(Builder);
			return Builder;
		}
	};

	/**
	 * Adapter for formatting a source location as file and line only.
	 */
	struct FFileAndLineAdapter
	{
		const FSourceLocation& Location;

		/**
		 * Converts the source location to a readable format with file and line information.
		 *
		 * @return String representation of the source location.
		 */
		[[nodiscard]] CORE_API FString ToString() const;
		[[nodiscard]] CORE_API FUtf8String ToUtf8String() const;

		/**
		 * Converts the source location to a readable format with file and line information, in place.
		 *
		 * @param Out String to fill with the string representation of the source location.
		 */
		CORE_API void ToString(FWideString& Out) const;
		CORE_API void ToString(FUtf8String& Out) const;

		/**
		 * Converts the source location to a readable format with file and line information, appending to an existing string.
		 *
		 * @param Out StringBuilder to append with the string representation of the source location.
		 */
		CORE_API void AppendString(FWideStringBuilderBase& Out) const;
		CORE_API void AppendString(FUtf8StringBuilderBase& Out) const;

		friend void SerializeForLog(FCbWriter& Writer, const FFileAndLineAdapter& Adapter)
		{
			Private::SerializeForLogFileAndLine(Writer, Adapter.Location);
		}

		friend FWideStringBuilderBase& operator<<(FWideStringBuilderBase& Builder, const FFileAndLineAdapter& Adapter)
		{
			Adapter.AppendString(Builder);
			return Builder;
		}
		friend FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Builder, const FFileAndLineAdapter& Adapter)
		{
			Adapter.AppendString(Builder);
			return Builder;
		}
	};

	/**
	 * Returns an adapter that formats source location with full information (file name, line, column, and function name).
	 *
	 * @code
	 *   // Example with string:
	 *   FString LocationStr = UE::SourceLocation::Full(SourceLoc).ToString();
	 *
	 *   // Example with string builder:
	 *   Builder << "Error at " << UE::SourceLocation::Full(SourceLoc);
	 *
	 *   // Example with structured log:
	 *   UE_LOGFMT(LogCore, Warning, "Error at {Location}", UE::SourceLocation::Full(SourceLoc));
	 * @endcode
	 *
	 * @param Location The source location to adapt.
	 * @return Adapter for full source location formatting.
	 */
	inline FFullAdapter Full(const FSourceLocation& Location)
	{
		return { Location };
	}

	/**
	 * Returns an adapter that formats source location with file and line only.
	 *
	 * @code
	 *   // Example with string:
	 *   FString LocationStr = UE::SourceLocation::FileAndLine(SourceLoc).ToString();
	 *
	 *   // Example with string builder:
	 *   Builder << "Error at " << UE::SourceLocation::FileAndLine(SourceLoc);
	 *
	 *   // Example with structured log:
	 *   UE_LOGFMT(LogCore, Warning, "Error at {Location}", UE::SourceLocation::FileAndLine(SourceLoc));
	 * @endcode
	 *
	 * @param Location The source location to adapt.
	 * @return Adapter for file and line formatting.
	 */
	inline FFileAndLineAdapter FileAndLine(const FSourceLocation& Location)
	{
		return { Location };
	}
}  // namespace UE::SourceLocation
