// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformProcess.h"
#include <atomic>

namespace UE
{
	// A mutex that doesn't put the thread into a WAIT state but instead repeatedly tries to aquire the lock.
	// WARNING: Should be used only for very short locks
	// Use with `TScopeLock`
	class FSpinLock
	{
	public:
		UE_NONCOPYABLE(FSpinLock);

		FSpinLock() = default;

		void Lock()
		{
			while (true)
			{
				if (!bFlag.exchange(true, std::memory_order_acquire))
				{
					break;
				}

				while (bFlag.load(std::memory_order_relaxed))
				{
					FPlatformProcess::Yield();
				}
			}
		}

		bool TryLock()
		{
			return !bFlag.exchange(true, std::memory_order_acquire);
		}

		void Unlock()
		{
			bFlag.store(false, std::memory_order_release);
		}

	private:
		std::atomic<bool> bFlag{ false };
	};
}
