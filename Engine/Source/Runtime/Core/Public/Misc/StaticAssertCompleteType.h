// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include <utility>

namespace UEStaticAssertCompleteType_Private
{
	class FIncompleteType;

#if defined(__clang__) || defined(__GNUC__)
	// Under Clang, a zero-sized array cannot be used in a template specialization, and also under GCC cannot be used as a function overload
	// or used with traits like std::is_array or std::remove_extent.
	// Testing for sizeof(T) == 0 works though, and should be the only type this is true for, then operator+ is used to coerce this into a
	// pointer and dereferenced to give a reference which can be used to check the completeness of the element type of the array.
	template <typename T, SIZE_T SizeOfT = sizeof(T)>
	struct TUEStaticAssertTypeCheckerBase
	{
		static T& Func();
	};
	template <typename T>
	struct TUEStaticAssertTypeCheckerBase<T, 0>
	{
		static decltype(*+std::declval<T>())& Func();
	};
#else
	// The above trick doesn't work for zero-sized arrays under MSVC, where sizeof(T[0]) is illegal, but specialization does.
	template <typename T>
	struct TUEStaticAssertTypeCheckerBase
	{
		static T& Func();
	};
	template <typename T>
	struct TUEStaticAssertTypeCheckerBase<T[0]>
	{
		static T& Func();
	};
#endif

	template <typename T>
	struct TUEStaticAssertTypeChecker : TUEStaticAssertTypeCheckerBase<T>
	{
	};

	// Voids will give an unhelpful error message when trying to take a reference to them, so work around it
	template <> struct TUEStaticAssertTypeChecker<               void> { static FIncompleteType Func(); };
	template <> struct TUEStaticAssertTypeChecker<const          void> { static FIncompleteType Func(); };
	template <> struct TUEStaticAssertTypeChecker<volatile       void> { static FIncompleteType Func(); };
	template <> struct TUEStaticAssertTypeChecker<const volatile void> { static FIncompleteType Func(); };

	// References are always complete types, but as we're using sizeof to check and sizeof ignores
	// references, let's just return a known complete type to make it work.
	template <typename T> struct TUEStaticAssertTypeChecker<T&>  { static int Func(); };
	template <typename T> struct TUEStaticAssertTypeChecker<T&&> { static int Func(); };

	// Function types are complete types, but we can't form a reference to one, so let's just make those work too
	template <typename RetType, typename... ArgTypes> struct TUEStaticAssertTypeChecker<RetType(ArgTypes...)> { static int Func(); };
}

// Causes a compile error if a type is incomplete
#define UE_STATIC_ASSERT_COMPLETE_TYPE(TypeToCheck, ...) static_assert(sizeof(UEStaticAssertCompleteType_Private::TUEStaticAssertTypeChecker<TypeToCheck>::Func()), ##__VA_ARGS__)

// Tests

// Each of these should fail to compile
#if 0
	UE_STATIC_ASSERT_COMPLETE_TYPE(               void,                                    "CV void is incomplete");
	UE_STATIC_ASSERT_COMPLETE_TYPE(      volatile void,                                    "CV void is incomplete");
	UE_STATIC_ASSERT_COMPLETE_TYPE(const          void,                                    "CV void is incomplete");
	UE_STATIC_ASSERT_COMPLETE_TYPE(const volatile void,                                    "CV void is incomplete");
	UE_STATIC_ASSERT_COMPLETE_TYPE(UEStaticAssertCompleteType_Private::FIncompleteType,    "A forward-declared-but-undefined class is incomplete");
	UE_STATIC_ASSERT_COMPLETE_TYPE(UEStaticAssertCompleteType_Private::FIncompleteType[2], "An array of an incomplete class is incomplete");
	UE_STATIC_ASSERT_COMPLETE_TYPE(int[],                                                  "An array of a complete type of unspecified bound is incomplete");
	UE_STATIC_ASSERT_COMPLETE_TYPE(UEStaticAssertCompleteType_Private::FIncompleteType[0], "A zero-sized array of an incomplete type is incomplete");
#endif

// Each of these should pass
#if 0
	UE_STATIC_ASSERT_COMPLETE_TYPE(UEStaticAssertCompleteType_Private::FIncompleteType*,                                                        "A pointer to an incomplete type is complete");
	UE_STATIC_ASSERT_COMPLETE_TYPE(UEStaticAssertCompleteType_Private::FIncompleteType&,                                                        "A reference to an incomplete type is complete");
	UE_STATIC_ASSERT_COMPLETE_TYPE(UEStaticAssertCompleteType_Private::FIncompleteType   (UEStaticAssertCompleteType_Private::FIncompleteType), "A function type is not incomplete, even if it returns or takes an incomplete type");
	UE_STATIC_ASSERT_COMPLETE_TYPE(UEStaticAssertCompleteType_Private::FIncompleteType(&)(UEStaticAssertCompleteType_Private::FIncompleteType), "References to function types must give a good error");
	UE_STATIC_ASSERT_COMPLETE_TYPE(int[0],                                                                                                      "A zero-sized array of a complete type is complete");
#endif
