// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h" // IWYU pragma: export
#include "Containers/StringView.h"
#include "CoreTypes.h"
#include "HAL/PlatformString.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Templates/EnableIf.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Templates/IsValidVariadicFunctionArg.h"
#include "Templates/Requires.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "Traits/IsCharEncodingSimplyConvertibleTo.h"
#include "Traits/IsCharType.h"
#include "Traits/IsContiguousContainer.h"

#include <iterator>
#include <type_traits>

template <typename T> struct TIsContiguousContainer;

namespace UE::Core::Private
{

class FDynamicStringBuilderToken
{
	explicit FDynamicStringBuilderToken() = default;
	template <typename CharType> friend class ::TStringBuilderBase;
	template <typename CharType, int32 BufferSize> friend class ::TStringBuilderWithBuffer;
};

class FExternalStringBuilderToken
{
	explicit FExternalStringBuilderToken() = default;
	template <typename CharType> friend class ::TExternalStringBuilder;
};

} // UE::Core::Private

/**
 * String Builder
 *
 * This class helps with the common task of constructing new strings.
 *
 * It does this by allocating buffer space which is used to hold the
 * constructed string. The intent is that the builder is allocated on
 * the stack as a function local variable to avoid heap allocations.
 *
 * The buffer is always contiguous and the class is not intended to be
 * used to construct extremely large strings.
 *
 * This is not intended to be used as a mechanism for holding on to
 * strings for a long time. The use case is explicitly to aid in
 * *constructing* strings on the stack and subsequently passing the
 * string into a function call or a more permanent string storage
 * mechanism like FString et al.
 *
 * The amount of buffer space to allocate is specified via a template
 * parameter and if the constructed string should overflow this initial
 * buffer, a new buffer will be allocated using regular dynamic memory
 * allocations.
 *
 * Overflow allocation should be the exceptional case however -- always
 * try to size the buffer so that it can hold the vast majority of
 * strings you expect to construct.
 *
 * Be mindful that stack is a limited resource, so if you are writing a
 * highly recursive function you may want to use some other mechanism
 * to build your strings.
 */
template <typename CharType>
class TStringBuilderBase
{
public:
	/** The character type that this builder operates on. */
	using ElementType = CharType;
	/** The string builder base type to be used by append operators and function output parameters. */
	using BuilderType = TStringBuilderBase<ElementType>;
	/** The string view type that this builder is compatible with. */
	using ViewType = TStringView<ElementType>;

	UE_DEPRECATED(5.8, "Use TStringBuilder<N> or T[CharType]StringBuilder<N> or TExternalStringBuilder<CharType>.")
	TStringBuilderBase()
	{
	}

	CORE_API	~TStringBuilderBase();

				TStringBuilderBase(const TStringBuilderBase&) = delete;
				TStringBuilderBase(TStringBuilderBase&&) = delete;

	TStringBuilderBase& operator=(const TStringBuilderBase&) = delete;
	TStringBuilderBase& operator=(TStringBuilderBase&&) = delete;

	TStringBuilderBase& operator=(ViewType Str)
	{
		Reset();
		return Append(Str);
	}
	
	TStringBuilderBase& operator=(const CharType* Str)
	{
		return *this = ViewType(Str);
	}

	UE_DEPRECATED(5.7, "Use TStringBuilder<N> or T[CharType]StringBuilder<N> or TExternalStringBuilder<CharType>.")
	inline TStringBuilderBase(CharType* BufferPointer, int32 BufferCapacity)
		: TStringBuilderBase(BufferPointer, BufferCapacity, UE::Core::Private::FDynamicStringBuilderToken{})
	{
	}

	inline int32 Len() const
	{
		return int32(CurPos - Base);
	}

	/** Returns a pointer to Len() code units that are not necessarily null-terminated. */
	inline CharType* GetData() UE_LIFETIMEBOUND
	{
		return Base;
	}

	inline const CharType* GetData() const UE_LIFETIMEBOUND
	{
		return Base;
	}

	/**
	 * Prefer operator*() for a pointer to a null-terminated string.
	 *
	 * Construct FString and other string types directly from the string builder: FString Str(Builder).
	 */
	inline const CharType* ToString() UE_LIFETIMEBOUND
	{
		EnsureNulTerminated();
		return Base;
	}

	/** Returns a pointer to a null-terminated string that is valid until the builder is mutated. */
	inline const CharType* operator*() UE_LIFETIMEBOUND
	{
		EnsureNulTerminated();
		return Base;
	}

	/** Returns a view of the string that is valid until the builder is mutated. */
	inline ViewType ToView() const UE_LIFETIMEBOUND
	{
		return ViewType(Base, Len());
	}

	/** Returns the last character, technically the last code unit. */
	inline CharType LastChar() const
	{
		return *(CurPos - 1);
	}

	/**
	 * Helper function to return the amount of memory allocated by this container.
	 * Does not include the sizeof of the inline buffer, only includes the size of the overflow buffer.
	 *
	 * @returns Number of bytes allocated by this container.
	 */
	SIZE_T GetAllocatedSize() const
	{
		return bIsDynamic ? (End - Base) * sizeof(CharType) : 0;
	}

	/**
	 * Empties the string builder, but doesn't change memory allocation.
	 */
	inline void Reset()
	{
		CurPos = Base;
	}

	/**
	 * Adds a given number of uninitialized characters into the string builder.
	 *
	 * @param InCount The number of uninitialized characters to add.
	 *
	 * @return The number of characters in the string builder before adding the new characters.
	 */
	inline int32 AddUninitialized(int32 InCount)
	{
		EnsureAdditionalCapacity(InCount);
		const int32 OldCount = Len();
		CurPos += InCount;
		return OldCount;
	}

	/**
	 * Reserves memory such that the string builder can contain at least the given number of characters.
	 *
	 * @note: Unlike AddUninitialized, this will not advance CurPos, and thus the various Append calls
	 * will start at the end of the string builder (from prior to this Reserve call).
	 * @param InNewCapacity The total number of characters the string builder can contain.
	 */
	inline void Reserve(int32 InNewCapacity)
	{
		checkSlow(InNewCapacity >= 0);
		const int32 CurrentCapacity = (int32)(End - Base);
		if (CurrentCapacity < InNewCapacity)
		{
			Extend(InNewCapacity - CurrentCapacity);
		}
	}

	/**
	 * Modifies the string builder to remove the given number of characters from the end.
	 */
	inline void RemoveSuffix(int32 InCount)
	{
		check(InCount <= Len());
		CurPos -= InCount;
	}

	template <typename OtherCharType,
		std::enable_if_t<TIsCharType<OtherCharType>::Value>* = nullptr>
	inline BuilderType& Append(const OtherCharType* const String, const int32 Length)
	{
		int32 ConvertedLength = FPlatformString::ConvertedLength<CharType>(String, Length);
		EnsureAdditionalCapacity(ConvertedLength);
		if (Length)
		{
			CurPos = FPlatformString::Convert(CurPos, ConvertedLength, String, Length);
		}
		return *this;
	}

	template <typename CharRangeType>
	inline auto Append(CharRangeType&& Range) -> decltype(Append(MakeStringView(Forward<CharRangeType>(Range)).GetData(), int32(0)))
	{
		const TStringView View = MakeStringView(Forward<CharRangeType>(Range));
		return Append(View.GetData(), View.Len());
	}

	template <
		typename AppendedCharType,
		std::enable_if_t<TIsCharType<AppendedCharType>::Value>* = nullptr
	>
	inline BuilderType& AppendChar(AppendedCharType Char)
	{
		if constexpr (TIsCharEncodingSimplyConvertibleTo_V<AppendedCharType, CharType>)
		{
			EnsureAdditionalCapacity(1);
			*CurPos++ = (CharType)Char;
		}
		else
		{
			int32 ConvertedLength = FPlatformString::ConvertedLength<CharType>(&Char, 1);
			EnsureAdditionalCapacity(ConvertedLength);
			CurPos = FPlatformString::Convert(CurPos, ConvertedLength, &Char, 1);
		}
		return *this;
	}

	/** Replace characters at given position and length with substring */
	void ReplaceAt(int32 Pos, int32 RemoveLen, ViewType Str)
	{
		check(Pos >= 0);
		check(RemoveLen >= 0);
		check(Pos + RemoveLen <= Len());

		const int DeltaLen = Str.Len() - RemoveLen;		
		if (DeltaLen < 0)
		{
			CurPos += DeltaLen;

			for (CharType* It = Base + Pos, *NewEnd = CurPos; It != NewEnd; ++It)
			{
				*It = *(It - DeltaLen);
			}
		}
		else if (DeltaLen > 0)
		{
			EnsureAdditionalCapacity(DeltaLen);
			CurPos += DeltaLen;

			for (CharType* It = CurPos - 1, *StopIt = Base + Pos + Str.Len() - 1; It != StopIt; --It)
			{
				*It = *(It - DeltaLen);
			}
		}
		
		if (Str.Len())
		{
			Str.CopyString(Base + Pos, Str.Len());
		}
	}

	/** Insert substring at given position */
	void InsertAt(int32 Pos, ViewType Str)
	{
		ReplaceAt(Pos, 0, Str);
	}

	/** Remove characters at given position */
	void RemoveAt(int32 Pos, int32 RemoveLen)
	{
		ReplaceAt(Pos, RemoveLen, ViewType());
	}

	/** Insert prefix */
	void Prepend(ViewType Str)
	{
		ReplaceAt(0, 0, Str);
	}

	/** Prefer AppendChar. Appends one character to the string. Supports this being the output for Algo::... */
	template <typename AppendedCharType>
	inline auto Add(AppendedCharType Char) -> decltype(AppendChar(Char), (void)0)
	{
		AppendChar(Char);
	}

	/**
	 * Append every element of the range to the builder, separating the elements by the delimiter.
	 *
	 * This function is only available when the elements of the range and the delimiter can both be
	 * written to the builder using the append operator.
	 *
	 * @param InRange The range of elements to join and append.
	 * @param InDelimiter The delimiter to append as a separator for the elements.
	 *
	 * @return The builder, to allow additional operations to be composed with this one.
	 */
	template <typename RangeType, typename DelimiterType>
		requires requires (BuilderType& Builder, RangeType&& Range, const DelimiterType& Delimiter)
		{
			Builder << *std::begin(Range);
			Builder << Delimiter;
		}
	inline BuilderType& Join(RangeType&& InRange, const DelimiterType& InDelimiter)
	{
		bool bFirst = true;
		for (auto&& Elem : Forward<RangeType>(InRange))
		{
			if (bFirst)
			{
				bFirst = false;
			}
			else
			{
				*this << InDelimiter;
			}
			*this << Elem;
		}
		return *this;
	}

	/**
	 * Append every element of the range to the builder, separating the elements by the delimiter, and
	 * surrounding every element on each side with the given quote.
	 *
	 * This function is only available when the elements of the range, the delimiter, and the quote can be
	 * written to the builder using the append operator.
	 *
	 * @param InRange The range of elements to join and append.
	 * @param InDelimiter The delimiter to append as a separator for the elements.
	 * @param InQuote The quote to append on both sides of each element.
	 *
	 * @return The builder, to allow additional operations to be composed with this one.
	 */
	template <typename RangeType, typename DelimiterType, typename QuoteType>
		requires requires (BuilderType& Builder, RangeType&& Range, const DelimiterType& Delimiter, const QuoteType& Quote)
	{
		Builder << Quote << *std::begin(Range) << Quote;
		Builder << Delimiter;
	}
	inline BuilderType& JoinQuoted(RangeType&& InRange, const DelimiterType& InDelimiter, const QuoteType& InQuote)
	{
		bool bFirst = true;
		for (auto&& Elem : Forward<RangeType>(InRange))
		{
			if (bFirst)
			{
				bFirst = false;
			}
			else
			{
				*this << InDelimiter;
			}
			*this << InQuote << Elem << InQuote;
		}
		return *this;
	}

private:
	template <typename SrcEncoding>
	using TIsCharEncodingCompatibleWithCharType = TIsCharEncodingCompatibleWith<SrcEncoding, CharType>;

public:
	/**
	 * Appends to the string builder similarly to how classic sprintf works.
	 *
	 * @param Fmt A format string that specifies how to format the additional arguments. Refer to standard printf format.
	 */
	template <typename FmtType, typename... Types
		UE_REQUIRES(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithCharType>::Value)>
	BuilderType& Appendf(const FmtType& Fmt, Types... Args)
	{
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to Appendf.");
		return AppendfImpl(*this, (const CharType*)Fmt, Forward<Types>(Args)...);
	}

	/**
	 * Appends to the string builder similarly to how classic vsprintf works.
	 *
	 * @param Fmt A format string that specifies how to format the additional arguments. Refer to standard printf format.
	 */
	CORE_API BuilderType& AppendV(const CharType* Fmt, va_list Args);

private:
	CORE_API static BuilderType& VARARGS AppendfImpl(BuilderType& Self, const CharType* Fmt, ...);

	friend inline       CharType* begin(      TStringBuilderBase& Builder) { return Builder.GetData(); }
	friend inline const CharType* begin(const TStringBuilderBase& Builder) { return Builder.GetData(); }
	friend inline       CharType* end  (      TStringBuilderBase& Builder) { return Builder.GetData() + Builder.Len(); }
	friend inline const CharType* end  (const TStringBuilderBase& Builder) { return Builder.GetData() + Builder.Len(); }

protected:
	inline TStringBuilderBase(CharType* InBase, int32 InCapacity, UE::Core::Private::FDynamicStringBuilderToken)
		: Base(InBase)
		, CurPos(InBase)
		, End(InBase + InCapacity)
	{
	}

	inline TStringBuilderBase(CharType* InBase, int32 InCapacity, UE::Core::Private::FExternalStringBuilderToken)
		: Base(InBase)
		, CurPos(InBase)
		, End(InBase + InCapacity)
		, bIsExternal(true)
	{
	}

	UE_DEPRECATED(5.7, "Use TStringBuilder<N> or T[CharType]StringBuilder<N> or TExternalStringBuilder<CharType>.")
	inline void Initialize(CharType* InBase, int32 InCapacity)
	{
		Base	= InBase;
		CurPos	= InBase;
		End		= Base + InCapacity;
	}

	inline void EnsureNulTerminated()
	{
		EnsureAdditionalCapacity(1);
		*CurPos = CharType(0);
	}

	inline void EnsureAdditionalCapacity(int32 RequiredAdditionalCapacity)
	{
		// precondition: we know the current buffer has enough capacity for the existing string

		if (LIKELY((CurPos + RequiredAdditionalCapacity) <= End))
		{
			return;
		}

		Extend(RequiredAdditionalCapacity);
	}

	CORE_API void Extend(SIZE_T ExtraCapacity);

	void*	AllocBuffer(SIZE_T CharCount);
	void	FreeBuffer(void* Buffer);

	CharType*	Base = nullptr;
	CharType*	CurPos = nullptr;
	CharType*	End = nullptr;
	bool		bIsDynamic = false;
	bool		bIsExternal = false;
};

template <typename CharType>
constexpr inline int32 GetNum(const TStringBuilderBase<CharType>& Builder)
{
	return Builder.Len();
}

//////////////////////////////////////////////////////////////////////////

/**
 * A string builder with inline storage.
 *
 * Avoid using this type directly. Prefer the aliases in StringFwd.h like TStringBuilder<N>.
 */
template <typename CharType, int32 BufferSize>
class TStringBuilderWithBuffer : public TStringBuilderBase<CharType>
{
public:
	inline TStringBuilderWithBuffer()
		: TStringBuilderBase<CharType>(StringBuffer, BufferSize, UE::Core::Private::FDynamicStringBuilderToken{})
	{
	}

	/**
	 * Construct a string builder by appending the arguments using operator<<.
	 */
	template <typename... ArgTypes>
	[[nodiscard]] inline explicit TStringBuilderWithBuffer(EInPlace, ArgTypes&&... Args)
		: TStringBuilderBase<CharType>(StringBuffer, BufferSize, UE::Core::Private::FDynamicStringBuilderToken{})
	{
		(*this << ... << (ArgTypes&&)Args);
	}

	using TStringBuilderBase<CharType>::operator=;

private:
	CharType StringBuffer[BufferSize > 0 ? BufferSize : 1];
};

//////////////////////////////////////////////////////////////////////////

/**
 * A string builder with fixed-size external storage.
 *
 * Asserts if external storage is exhausted.
 */
template <typename CharType>
class TExternalStringBuilder : public TStringBuilderBase<CharType>
{
public:
	[[nodiscard]] inline explicit TExternalStringBuilder(CharType* Buffer, int32 BufferSize)
		: TStringBuilderBase<CharType>(Buffer, BufferSize, UE::Core::Private::FExternalStringBuilderToken{})
	{
	}

	using TStringBuilderBase<CharType>::operator=;
};

//////////////////////////////////////////////////////////////////////////

// String Append Operators

template <typename CharType, typename CharRangeType>
inline auto operator<<(TStringBuilderBase<CharType>& Builder, CharRangeType&& Str) -> decltype(Builder.Append(MakeStringView(Forward<CharRangeType>(Str))))
{
	// Anything convertible to an FAnsiStringView is also convertible to a FUtf8StringView, but FAnsiStringView is more efficient to convert
	if constexpr (std::is_convertible_v<CharRangeType, FAnsiStringView>)
	{
		return Builder.Append(ImplicitConv<FAnsiStringView>(Forward<CharRangeType>(Str)));
	}
	else
	{
		return Builder.Append(MakeStringView(Forward<CharRangeType>(Str)));
	}
}

inline FAnsiStringBuilderBase&		operator<<(FAnsiStringBuilderBase& Builder, ANSICHAR Char)							{ return Builder.AppendChar(Char); }
inline FAnsiStringBuilderBase&		operator<<(FAnsiStringBuilderBase& Builder, UTF8CHAR Char) = delete;
inline FAnsiStringBuilderBase&		operator<<(FAnsiStringBuilderBase& Builder, WIDECHAR Char) = delete;
inline FWideStringBuilderBase&		operator<<(FWideStringBuilderBase& Builder, ANSICHAR Char)							{ return Builder.AppendChar(Char); }
inline FWideStringBuilderBase&		operator<<(FWideStringBuilderBase& Builder, UTF8CHAR Char) = delete;
inline FWideStringBuilderBase&		operator<<(FWideStringBuilderBase& Builder, WIDECHAR Char)							{ return Builder.AppendChar(Char); }
inline FUtf8StringBuilderBase&		operator<<(FUtf8StringBuilderBase& Builder, ANSICHAR Char)							{ return Builder.AppendChar(UTF8CHAR(Char)); }
inline FUtf8StringBuilderBase&		operator<<(FUtf8StringBuilderBase& Builder, UTF8CHAR Char)							{ return Builder.AppendChar(Char); }
inline FUtf8StringBuilderBase&		operator<<(FUtf8StringBuilderBase& Builder, WIDECHAR Char) = delete;
inline FWideStringBuilderBase&		operator<<(FWideStringBuilderBase& Builder, UTF32CHAR Char)							{ return Builder.AppendChar(Char); }
inline FUtf8StringBuilderBase&		operator<<(FUtf8StringBuilderBase& Builder, UTF32CHAR Char)							{ return Builder.AppendChar(Char); }

// Prefer using << instead of += as operator+= is only intended for mechanical FString -> FStringView replacement.
inline FStringBuilderBase&			operator+=(FStringBuilderBase& Builder, ANSICHAR Char)								{ return Builder.AppendChar(Char); }
inline FStringBuilderBase&			operator+=(FStringBuilderBase& Builder, WIDECHAR Char)								{ return Builder.AppendChar(Char); }
inline FStringBuilderBase&			operator+=(FStringBuilderBase& Builder, UTF8CHAR Char)								{ return Builder.AppendChar(Char); }
inline FStringBuilderBase&			operator+=(FStringBuilderBase& Builder, FWideStringView Str)						{ return Builder.Append(Str); }
inline FStringBuilderBase&			operator+=(FStringBuilderBase& Builder, FUtf8StringView Str)						{ return Builder.Append(Str); }

// Bool Append Operators

template <typename T UE_REQUIRES(std::is_same_v<bool, T>)>
inline FAnsiStringBuilderBase&		operator<<(FAnsiStringBuilderBase& Builder, T Value)								{ return Builder.Append(Value ? "true" : "false"); }
template <typename T UE_REQUIRES(std::is_same_v<bool, T>)>
inline FWideStringBuilderBase&		operator<<(FWideStringBuilderBase& Builder, T Value)								{ return Builder.Append(Value ? WIDETEXT("true") : WIDETEXT("false")); }
template <typename T UE_REQUIRES(std::is_same_v<bool, T>)>
inline FUtf8StringBuilderBase&		operator<<(FUtf8StringBuilderBase& Builder, T Value)								{ return Builder.Append(Value ? UTF8TEXT("true") : UTF8TEXT("false")); }

/**
 * Trait which determines whether or not a type can be appended to TStringBuilderBase via TFormatSpecifier.
 */

template <typename T>        constexpr bool TIsFormattedStringBuilderType_V                   = false;
template <>           inline constexpr bool TIsFormattedStringBuilderType_V<uint8>            = true;
template <>           inline constexpr bool TIsFormattedStringBuilderType_V<uint16>           = true;
template <>           inline constexpr bool TIsFormattedStringBuilderType_V<uint32>           = true;
template <>           inline constexpr bool TIsFormattedStringBuilderType_V<uint64>           = true;
template <>           inline constexpr bool TIsFormattedStringBuilderType_V<int8>             = true;
template <>           inline constexpr bool TIsFormattedStringBuilderType_V<int16>            = true;
template <>           inline constexpr bool TIsFormattedStringBuilderType_V<int32>            = true;
template <>           inline constexpr bool TIsFormattedStringBuilderType_V<int64>            = true;
template <>           inline constexpr bool TIsFormattedStringBuilderType_V<float>            = true;
template <>           inline constexpr bool TIsFormattedStringBuilderType_V<double>           = true;
template <>           inline constexpr bool TIsFormattedStringBuilderType_V<long double>      = true;
template <>           inline constexpr bool TIsFormattedStringBuilderType_V<long>             = true;
template <>           inline constexpr bool TIsFormattedStringBuilderType_V<unsigned long>    = true;

template <typename T>        constexpr bool TIsFormattedStringBuilderType_V<const          T> = TIsFormattedStringBuilderType_V<T>;
template <typename T>        constexpr bool TIsFormattedStringBuilderType_V<      volatile T> = TIsFormattedStringBuilderType_V<T>;
template <typename T>        constexpr bool TIsFormattedStringBuilderType_V<const volatile T> = TIsFormattedStringBuilderType_V<T>;

// Formatted Append Operators

template <
	typename CharType,
	typename T
	UE_REQUIRES(TIsFormattedStringBuilderType_V<T>)
>
inline TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const T& Value)
{
	// std::remove_cv_t to remove potential volatile decorations. Removing const is pointless, but harmless because it's specified in the param declaration.
	return Builder.Appendf(TFormatSpecifier<std::remove_cv_t<T>>::template GetFormatSpecifier<CharType>(), Value);
}

/**
 * A function to create and append to a temporary string builder.
 *
 * Example Use Cases:
 *
 * For void Action(FStringView) -> Action(WriteToString<64>(Arg1, Arg2));
 * For UE_LOG or checkf -> checkf(Condition, TEXT("%s"), *WriteToString<32>(Arg));
 */
template <int32 BufferSize, typename... ArgTypes>
TStringBuilderWithBuffer<TCHAR, BufferSize> WriteToString(ArgTypes&&... Args)
{
	return TStringBuilderWithBuffer<TCHAR, BufferSize>(InPlace, (ArgTypes&&)Args...);
}

/** A function to create and append to a temporary string builder. See WriteToString. */
template <int32 BufferSize, typename... ArgTypes>
TStringBuilderWithBuffer<ANSICHAR, BufferSize> WriteToAnsiString(ArgTypes&&... Args)
{
	return TStringBuilderWithBuffer<ANSICHAR, BufferSize>(InPlace, (ArgTypes&&)Args...);
}

/** A function to create and append to a temporary string builder. See WriteToString. */
template <int32 BufferSize, typename... ArgTypes>
TStringBuilderWithBuffer<WIDECHAR, BufferSize> WriteToWideString(ArgTypes&&... Args)
{
	return TStringBuilderWithBuffer<WIDECHAR, BufferSize>(InPlace, (ArgTypes&&)Args...);
}

/** A function to create and append to a temporary string builder. See WriteToString. */
template <int32 BufferSize, typename... ArgTypes>
TStringBuilderWithBuffer<UTF8CHAR, BufferSize> WriteToUtf8String(ArgTypes&&... Args)
{
	return TStringBuilderWithBuffer<UTF8CHAR, BufferSize>(InPlace, (ArgTypes&&)Args...);
}

/**
 * Returns an object that can be used as the output container for algorithms by appending to the builder.
 *
 * Example: Algo::Transform(StringView, AppendChars(Builder), FChar::ToLower)
 */
template <typename CharType>
auto AppendChars(TStringBuilderBase<CharType>& Builder)
{
	struct FAppendChar
	{
		TStringBuilderBase<CharType>& Builder;
		inline void Add(CharType Char) { Builder.AppendChar(Char); }
	};
	return FAppendChar{Builder};
}
