// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Included through UnrealString.h

#include "Containers/ContainersFwd.h"

/** An argument supplied to FString::Format */
struct FStringFormatArg
{
	enum EType { Int, UInt, Double, String, StringLiteralANSI, StringLiteralWIDE, StringLiteralUCS2, StringLiteralUTF8 };

	/** The type of this arg */
	EType Type;

	/* todo: convert this to a TVariant */
	union
	{
		/** Value as integer */
		int64 IntValue;
		/** Value as uint */
		uint64 UIntValue;
		/** Value as double */
		double DoubleValue;
		/** Value as an ANSI string literal */
		const ANSICHAR* StringLiteralANSIValue;
		/** Value as a WIDE string literal */
		const WIDECHAR* StringLiteralWIDEValue;
		/** Value as a UCS2 string literal */
		const UCS2CHAR* StringLiteralUCS2Value;
		/** Value as a UTF8 string literal */
		const UTF8CHAR* StringLiteralUTF8Value;
	};

	/** Value as an FString */
	FString StringValue;

	/** Not default constructible */
	FStringFormatArg() = delete;
	~FStringFormatArg() = default;

	CORE_API FStringFormatArg(const int32 Value);
	CORE_API FStringFormatArg(const uint32 Value);
	CORE_API FStringFormatArg(const int64 Value);
	CORE_API FStringFormatArg(const uint64 Value);
	CORE_API FStringFormatArg(const float Value);
	CORE_API FStringFormatArg(const double Value);
	CORE_API FStringFormatArg(FString Value);
	CORE_API FStringFormatArg(FStringView Value);
	CORE_API FStringFormatArg(const ANSICHAR* Value);
	CORE_API FStringFormatArg(const WIDECHAR* Value);
	CORE_API FStringFormatArg(const UCS2CHAR* Value);
	CORE_API FStringFormatArg(const UTF8CHAR* Value);

	FStringFormatArg(const FStringFormatArg& Other)
	{
		*this = Other;
	}
	FStringFormatArg(FStringFormatArg&& Other)
	{
		*this = MoveTemp(Other);
	}
	
	CORE_API FStringFormatArg& operator=(const FStringFormatArg& Other);
	CORE_API FStringFormatArg& operator=(FStringFormatArg&& Other);

};
