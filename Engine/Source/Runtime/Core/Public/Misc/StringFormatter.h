// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/ExpressionParserTypes.h"
#include "Templates/ValueOrError.h"

struct FStringFormatArg;

/** A string formatter is responsible for formatting string patterns using a set of named, or ordered arguments */
template <typename CharType>
class TStringFormatter
{
public:

	CORE_API TStringFormatter();

	/**
	 * Format the specified string using the specified arguments. Replaces instances of { Argument } with keys in the map matching 'Argument'
	 * @param InExpression		A string representing the format expression
	 * @param InArgs			A map of named arguments that match the tokens specified in InExpression
	 * @return A string containing the formatted text
	 */
	TString<CharType> Format(const CharType* InExpression, const TMap<TString<CharType>, FStringFormatArg>& InArgs) const
	{
		auto Result = FormatInternal(InExpression, InArgs, false);
		if (ensure(Result.IsValid()))
		{
			return MoveTemp(Result.GetValue());
		}

		return InExpression;
	}

	/**
	 * Format the specified string using the specified arguments. Replaces instances of {0} with indices from the given array matching the index specified in the token
	 * @param InExpression		A string representing the format expression
	 * @param InArgs			An array of ordered arguments that match the tokens specified in InExpression
	 * @return A string containing the formatted text
	 */
	TString<CharType> Format(const CharType* InExpression, const TArray<FStringFormatArg>& InArgs) const
	{
		auto Result = FormatInternal(InExpression, InArgs, false);
		if (ensure(Result.IsValid()))
		{
			return MoveTemp(Result.GetValue());
		}
		
		return InExpression;
	}

	/**
	 * Format the specified string using the specified arguments. Replaces instances of { Argument } with keys in the map matching 'Argument'
	 * @param InExpression		A string representing the format expression
	 * @param InArgs			A map of named arguments that match the tokens specified in InExpression
	 * @return A string containing the formatted text, or an error where InExpression is ill-formed, or contains undefined arguments
	*/
	TValueOrError<TString<CharType>, FExpressionError> FormatStrict(const CharType* InExpression, const TMap<TString<CharType>, FStringFormatArg>& InArgs) const
	{
		return FormatInternal(InExpression, InArgs, true);
	}

	/**
	 * Format the specified string using the specified arguments. Replaces instances of {0} with indices from the given array matching the index specified in the token
	 * @param InExpression		A string representing the format expression
	 * @param InArgs			An array of ordered arguments that match the tokens specified in InExpression
	 * @return A string containing the formatted text, or an error where InExpression is ill-formed, or contains undefined arguments
	 */
	TValueOrError<TString<CharType>, FExpressionError> FormatStrict(const CharType* InExpression, const TArray<FStringFormatArg>& InArgs) const
	{
		return FormatInternal(InExpression, InArgs, true);
	}

private:

	/** Internal formatting logic */
	CORE_API TValueOrError<TString<CharType>, FExpressionError> FormatInternal(const CharType* InExpression, const TMap<TString<CharType>, FStringFormatArg>& InArgs, bool bStrict) const;
	CORE_API TValueOrError<TString<CharType>, FExpressionError> FormatInternal(const CharType* InExpression, const TArray<FStringFormatArg>& InArgs, bool bStrict) const;

	/** Token definitions for lenient lexers */
	TTokenDefinitions<CharType> NamedDefinitions;
	TTokenDefinitions<CharType> OrderedDefinitions;

	/** Token definitions for strict lexers */
	TTokenDefinitions<CharType> StrictNamedDefinitions;
	TTokenDefinitions<CharType> StrictOrderedDefinitions;
};

extern template class TStringFormatter<ANSICHAR>;
extern template class TStringFormatter<UTF8CHAR>;
extern template class TStringFormatter<WIDECHAR>;
