// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/OutputDevice.h"
#include "Templates/Tuple.h"
#include "Traits/IsContiguousContainer.h"

class FName;

namespace ELogVerbosity
{
	enum Type : uint8;
}

/**
 * An output device which writes to an FString.
 */
class FStringOutputDevice : public FString, public FOutputDevice
{
public:
	FStringOutputDevice(const TCHAR* Prefix = TEXT(""))
		: FString(Prefix)
	{
		bAutoEmitLineTerminator = false;
	}

	virtual void Serialize(const TCHAR* InData, ELogVerbosity::Type Verbosity, const FName& Category) override
	{
		FString::operator+=((TCHAR*)InData);
		if (bAutoEmitLineTerminator)
		{
			*this += LINE_TERMINATOR;
		}
	}

	FStringOutputDevice(FStringOutputDevice&&) = default;
	FStringOutputDevice(const FStringOutputDevice&) = default;
	FStringOutputDevice& operator=(FStringOutputDevice&&) = default;
	FStringOutputDevice& operator=(const FStringOutputDevice&) = default;

	// Make += operator virtual.
	virtual FString& operator+=(const FString& Other)
	{
		return FString::operator+=(Other);
	}

	UE_REWRITE bool UEOpEquals(const FString& Rhs) const
	{
		return FString::UEOpEquals(Rhs);
	}

	UE_REWRITE bool UEOpLessThan(const FString& Rhs) const
	{
		return FString::UEOpLessThan(Rhs);
	}

	UE_REWRITE bool UEOpGreaterThan(const FString& Rhs) const
	{
		return Rhs.UEOpLessThan(*this);
	}
};

template <>
struct TIsContiguousContainer<FStringOutputDevice>
{
	enum { Value = true };
};

/**
 * An output device which counts lines as it builds up a string.
 */
class FStringOutputDeviceCountLines : public FStringOutputDevice
{
	using Super = FStringOutputDevice;

public:
	FStringOutputDeviceCountLines(const TCHAR* Prefix = TEXT(""))
		: Super(Prefix)
	{
	}

	virtual void Serialize(const TCHAR* InData, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		Super::Serialize(InData, Verbosity, Category);
		int32 TermLength = FCString::Strlen(LINE_TERMINATOR);
		for (;;)
		{
			InData = FCString::Strstr(InData, LINE_TERMINATOR);
			if (!InData)
			{
				break;
			}
			LineCount++;
			InData += TermLength;
		}

		if (bAutoEmitLineTerminator)
		{
			LineCount++;
		}
	}

	/**
	 * Appends other FStringOutputDeviceCountLines object to this one.
	 */
	virtual FStringOutputDeviceCountLines& operator+=(const FStringOutputDeviceCountLines& Other)
	{
		FString::operator+=(static_cast<const FString&>(Other));

		LineCount += Other.GetLineCount();

		return *this;
	}

	/**
	 * Appends other FString (as well as its specializations like FStringOutputDevice)
	 * object to this.
	 */
	virtual FString& operator+=(const FString& Other) override
	{
		Log(Other);

		return *this;
	}

	int32 GetLineCount() const
	{
		return LineCount;
	}

	FStringOutputDeviceCountLines(const FStringOutputDeviceCountLines&) = default;
	FStringOutputDeviceCountLines& operator=(const FStringOutputDeviceCountLines&) = default;

	inline FStringOutputDeviceCountLines(FStringOutputDeviceCountLines&& Other)
		: Super    ((Super&&)Other)
		, LineCount(Other.LineCount)
	{
		Other.LineCount = 0;
	}

	inline FStringOutputDeviceCountLines& operator=(FStringOutputDeviceCountLines&& Other)
	{
		if (this != &Other)
		{
			(Super&)*this = (Super&&)Other;
			LineCount     = Other.LineCount;

			Other.LineCount = 0;
		}
		return *this;
	}

private:
	int32 LineCount = 0;
};

template <>
struct TIsContiguousContainer<FStringOutputDeviceCountLines>
{
	enum { Value = true };
};

/**
 * Serializes log lines of the requested categories to a string; discards other log lines.
 */
template <typename... LogCategoryTypes>
class TLogCategoryOutputDevice : public FStringOutputDevice
{
public:
	explicit TLogCategoryOutputDevice(LogCategoryTypes&... InLogCategories UE_LIFETIMEBOUND)
		: LogCategories(&InLogCategories...)
	{
		bAutoEmitLineTerminator = true;
	}

	virtual void Serialize(const TCHAR* InData, ELogVerbosity::Type Verbosity, const FName& Category) override
	{
		bool bFoundMatchingCategory = false;
		VisitTupleElements(
			[&bFoundMatchingCategory, &Category](auto& Elem)
			{
				bFoundMatchingCategory = bFoundMatchingCategory || (Category == Elem->GetCategoryName());
			},
			LogCategories);

		if (bFoundMatchingCategory)
		{
			FStringOutputDevice::Serialize(InData, Verbosity, Category);
		}
	}

private:
	TTuple<LogCategoryTypes*...> LogCategories;
};

template <typename... LogCategoryTypes>
struct TIsContiguousContainer<TLogCategoryOutputDevice<LogCategoryTypes...>>
{
	enum { Value = true };
};
