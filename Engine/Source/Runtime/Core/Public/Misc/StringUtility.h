// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"

namespace StringUtility
{
	/**
	 * Unescapes a URI
	 *
	 * @param URLString an escaped string (e.g. File%20Name)
	 *
	 * @return un-escaped string (e.g. "File Name")
	 */
	FString CORE_API UnescapeURI(const FString& URLString);
}

