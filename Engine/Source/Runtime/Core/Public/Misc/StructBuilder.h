// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/AlignmentTemplates.h"

/**
 * Lays out up a struct at runtime, in hopefully the same way as a compiler,
 * and returns the offset of each member as it's added.
 */
class FStructBuilder
{
public:
	constexpr FStructBuilder()
		: EndOfLastMember(0)
		, Alignment      (0)
	{
	}

	/**
	 * Adds a new member to the end of the struct.
	 *
	 * @param  MemberSize       The size of the member.
	 * @param  MemberAlignment  The alignment of the member.
	 *
	 * @return  The offset of the member from the start of the struct.
	 */
	constexpr int32 AddMember(int32 MemberSize, int32 MemberAlignment)
	{
		int32 Result    = Align(EndOfLastMember, MemberAlignment);
		EndOfLastMember = Result + MemberSize;
		Alignment       = FMath::Max(Alignment, MemberAlignment);
		return Result;
	}

	/**
	 * Returns the current size of the struct.
	 *
	 * @return  The size of the struct.
	 */
	constexpr int32 GetSize() const
	{
		return Align(EndOfLastMember, Alignment);
	}

	/**
	 * Returns the current alignment of the struct.
	 *
	 * @return  The alignment of the struct.
	 */
	constexpr int32 GetAlignment() const
	{
		return Alignment;
	}

private:
	// The offset from the start of the struct to the end of the last added member
	int32 EndOfLastMember;

	// The alignment of the struct
	int32 Alignment;
};
