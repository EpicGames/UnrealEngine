// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Requires.h"
#include "TVariantMeta.h"
#include <type_traits>

/**
 * A special tag used to indicate that in-place construction of a variant should take place.
 */
template <typename T>
struct TInPlaceType {};

/**
 * INTERNAL USE ONLY -- Do not use this type.
 * Its internal use case is for in-place construction of non-default-constructible types during serialization.
 */
namespace UE::Core::Private
{
template <typename T>
struct TUninitializedType {};
}

/**
 * A special tag that can be used as the first type in a TVariant parameter pack if none of the other types can be default-constructed.
 */
struct FEmptyVariantState
{
	/** Allow FEmptyVariantState to be used with FArchive serialization */
	friend inline FArchive& operator<<(FArchive& Ar, FEmptyVariantState&)
	{
		return Ar;
	}
};

/**
 * A type-safe union based loosely on std::variant. This flavor of variant requires that all the types in the declaring template parameter pack be unique.
 * Attempting to use the value of a Get() when the underlying type is different leads to undefined behavior.
 */
template <typename T, typename... Ts>
class TVariant final : private UE::Core::Private::TVariantStorage<T, Ts...>
{
	using Super = UE::Core::Private::TVariantStorage<T, Ts...>;

	static_assert(!UE::Core::Private::TTypePackContainsDuplicates<T, Ts...>::Value, "All the types used in TVariant should be unique");
	static_assert(!UE::Core::Private::TContainsReferenceType<T, Ts...>::Value, "TVariant cannot hold reference types");

	// Test for 255 here, because the parameter pack doesn't include the initial T
	static_assert(sizeof...(Ts) <= 255, "TVariant cannot hold more than 256 types");

public:
	/** Default initialize the TVariant to the first type in the parameter pack */
	[[nodiscard]] TVariant()
	{
		static_assert(std::is_constructible_v<T>, "To default-initialize a TVariant, the first type in the parameter pack must be default constructible. Use FEmptyVariantState as the first type if none of the other types can be listed first.");
		::new((void*)&UE::Core::Private::CastToStorage(*this).Storage) T();
		TypeIndex = 0;
	}

	/** Perform in-place construction of a type into the variant */
	template <
		typename U,
		typename... ArgTypes
		UE_REQUIRES(std::is_constructible_v<U, ArgTypes...>)
	>
	[[nodiscard]] explicit TVariant(TInPlaceType<U>&&, ArgTypes&&... Args)
	{
		constexpr SIZE_T Index = UE::Core::Private::TParameterPackTypeIndex<U, T, Ts...>::Value;
		static_assert(Index != (SIZE_T)-1, "The TVariant is not declared to hold the type being constructed");

		::new((void*)&UE::Core::Private::CastToStorage(*this).Storage) U(Forward<ArgTypes>(Args)...);
		TypeIndex = (uint8)Index;
	}

	/** Copy construct the variant from another variant of the same type */
	[[nodiscard]] TVariant(const TVariant& Other)
	{
		TypeIndex = Other.TypeIndex;
		UE::Core::Private::TCopyConstructorLookup<T, Ts...>::Construct(TypeIndex, &UE::Core::Private::CastToStorage(*this).Storage, &UE::Core::Private::CastToStorage(Other).Storage);
	}

	/** Move construct the variant from another variant of the same type */
	[[nodiscard]] TVariant(TVariant&& Other)
	{
		TypeIndex = Other.TypeIndex;
		UE::Core::Private::TMoveConstructorLookup<T, Ts...>::Construct(TypeIndex, &UE::Core::Private::CastToStorage(*this).Storage, &UE::Core::Private::CastToStorage(Other).Storage);
	}

	/** Copy assign a variant from another variant of the same type */
	TVariant& operator=(const TVariant& Other)
	{
		if (&Other != this)
		{
			TVariant Temp = Other;
			Swap(Temp, *this);
		}
		return *this;
	}

	/** Move assign a variant from another variant of the same type */
	TVariant& operator=(TVariant&& Other)
	{
		if (&Other != this)
		{
			TVariant Temp = MoveTemp(Other);
			Swap(Temp, *this);
		}
		return *this;
	}

	/** Destruct the underlying type (if appropriate) */
	~TVariant()
		requires(!std::is_trivially_destructible_v<T> || (!std::is_trivially_destructible_v<Ts> || ...))
	{
		UE::Core::Private::TDestructorLookup<T, Ts...>::Destruct(TypeIndex, &UE::Core::Private::CastToStorage(*this).Storage);
	}
	~TVariant()
		requires(std::is_trivially_destructible_v<T> && (std::is_trivially_destructible_v<Ts> && ...))
	= default;

	/** Determine if the variant holds the specific type */
	template <typename U>
	[[nodiscard]] bool IsType() const
	{
		static_assert(UE::Core::Private::TParameterPackTypeIndex<U, T, Ts...>::Value != (SIZE_T)-1, "The TVariant is not declared to hold the type passed to IsType<>");
		return UE::Core::Private::TIsType<U, T, Ts...>::IsSame(TypeIndex);
	}

	/** Get a reference to the held value. Bad things can happen if this is called on a variant that does not hold the type asked for */
	template <typename U>
	[[nodiscard]] U& Get() UE_LIFETIMEBOUND
	{
		constexpr SIZE_T Index = UE::Core::Private::TParameterPackTypeIndex<U, T, Ts...>::Value;
		static_assert(Index != (SIZE_T)-1, "The TVariant is not declared to hold the type passed to Get<>");

		check(Index == TypeIndex);
		// The intermediate step of casting to void* is used to avoid warnings due to use of reinterpret_cast between related types if U and the storage class are related
		// This was specifically encountered when U derives from TAlignedBytes
		return *reinterpret_cast<U*>(reinterpret_cast<void*>(&UE::Core::Private::CastToStorage(*this).Storage));
	}

	/** Get a reference to the held value. Bad things can happen if this is called on a variant that does not hold the type asked for */
	template <typename U>
	[[nodiscard]] const U& Get() const UE_LIFETIMEBOUND
	{
		// Temporarily remove the const qualifier so we can implement Get in one location.
		return const_cast<TVariant*>(this)->template Get<U>();
	}

	/** Get a reference to the held value if set, otherwise the DefaultValue */
	template <typename U>
	const U& Get(const TIdentity_T<U>& DefaultValue UE_LIFETIMEBOUND) const UE_LIFETIMEBOUND
	{
		return IsType<U>() ? Get<U>() : DefaultValue;
	}

	/** Get a pointer to the held value if the held type is the same as the one specified */
	template <typename U>
	[[nodiscard]] U* TryGet() UE_LIFETIMEBOUND
	{
		constexpr SIZE_T Index = UE::Core::Private::TParameterPackTypeIndex<U, T, Ts...>::Value;
		static_assert(Index != (SIZE_T)-1, "The TVariant is not declared to hold the type passed to TryGet<>");
		// The intermediate step of casting to void* is used to avoid warnings due to use of reinterpret_cast between related types if U and the storage class are related
		// This was specifically encountered when U derives from TAlignedBytes
		return Index == (SIZE_T)TypeIndex ? reinterpret_cast<U*>(reinterpret_cast<void*>(&UE::Core::Private::CastToStorage(*this).Storage)) : nullptr;
	}

	/** Get a pointer to the held value if the held type is the same as the one specified */
	template <typename U>
	[[nodiscard]] const U* TryGet() const UE_LIFETIMEBOUND
	{
		// Temporarily remove the const qualifier so we can implement TryGet in one location.
		return const_cast<TVariant*>(this)->template TryGet<U>();
	}

	/** Set a specifically-typed value into the variant */
	template <typename U>
	void Set(typename TIdentity<U>::Type&& Value)
	{
		Emplace<U>(MoveTemp(Value));
	}

	/** Set a specifically-typed value into the variant */
	template <typename U>
	void Set(const typename TIdentity<U>::Type& Value)
	{
		Emplace<U>(Value);
	}

	/** Set a specifically-typed value into the variant using in-place construction */
	template <
		typename U,
		typename... ArgTypes
		UE_REQUIRES(std::is_constructible_v<U, ArgTypes...>)
	>
	void Emplace(ArgTypes&&... Args)
	{
		constexpr SIZE_T Index = UE::Core::Private::TParameterPackTypeIndex<U, T, Ts...>::Value;
		static_assert(Index != (SIZE_T)-1, "The TVariant is not declared to hold the type passed to Emplace<>");

		UE::Core::Private::TDestructorLookup<T, Ts...>::Destruct(TypeIndex, &UE::Core::Private::CastToStorage(*this).Storage);
		::new((void*)&UE::Core::Private::CastToStorage(*this).Storage) U(Forward<ArgTypes>(Args)...);
		TypeIndex = (uint8)Index;
	}

	/** Lookup the index of a type in the template parameter pack at compile time. */
	template <typename U>
	[[nodiscard]] static constexpr SIZE_T IndexOfType()
	{
		constexpr SIZE_T Index = UE::Core::Private::TParameterPackTypeIndex<U, T, Ts...>::Value;
		static_assert(Index != (SIZE_T)-1, "The TVariant is not declared to hold the type passed to IndexOfType<>");
		return Index;
	}

	/** Returns the currently held type's index into the template parameter pack */
	[[nodiscard]] SIZE_T GetIndex() const
	{
		return (SIZE_T)TypeIndex;
	}

	/** 
	 * INTERNAL USE ONLY -- Do not call this constructor, it will put the variant in a bad state.
	 * Its internal use case is for in-place construction of non-default-constructible types during serialization.
	 *
	 * Construct the TVariant to store the specified uninitialized element type.
	 * The caller must unconditionally get a pointer to the element and construct an object of the right type in that position.
	 * This is all totally exception-unsafe.
	 * This relies on undefined behavior by dereferencing a pointer to an object (inside Get<U>) that doesn't exist,
	 * and then turning it back to an pointer.
	 * Any other attempts to use or destroy the variant before an object of the right type has been constructed is completely unsafe,
	 * it may e.g. result in trying to destroy an object that is not there.
	 */
	template <typename U>
	[[nodiscard]] explicit TVariant(UE::Core::Private::TUninitializedType<U>&&)
	{
		constexpr SIZE_T Index = UE::Core::Private::TParameterPackTypeIndex<U, T, Ts...>::Value;
		static_assert(Index != (SIZE_T)-1, "The TVariant is not declared to store the specified type");
		TypeIndex = (uint8)Index;
	}

private:
	/** Index into the template parameter pack for the type held. */
	uint8 TypeIndex;
};

/** Apply a visitor function to the list of variants */
template <
	typename Func,
	typename... Variants
	UE_REQUIRES((TIsVariant_V<std::decay_t<Variants>> && ...))
>
decltype(auto) Visit(Func&& Callable, Variants&&... Args)
{
	constexpr SIZE_T NumPermutations = (1 * ... * (TVariantSize_V<std::decay_t<Variants>>));

	return UE::Core::Private::VisitImpl(
		UE::Core::Private::EncodeIndices(Args...),
		Forward<Func>(Callable),
		TMakeIntegerSequence<SIZE_T, NumPermutations>{},
		TMakeIntegerSequence<SIZE_T, sizeof...(Variants)>{},
		Forward<Variants>(Args)...
	);
}

/**
 * Serialization function for TVariants. 
 *
 * In order for a TVariant to be serializable, each type in its template parameter pack must:
 *   1. Have a default constructor. This is required because when reading the type from an archive, it must be default constructed before being loaded.
 *   2. Implement the `FArchive& operator<<(FArchive&, T&)` function. This is required to serialize the actual type that's stored in TVariant.
 */
template <typename... Ts>
inline FArchive& operator<<(typename UE::Core::Private::TAlwaysFArchive<TVariant<Ts...>>::Type& Ar, TVariant<Ts...>& Variant)
{
	if (Ar.IsLoading())
	{
		uint8 Index;
		Ar << Index;
		check(Index < sizeof...(Ts));

		UE::Core::Private::TVariantLoadFromArchiveLookup<Ts...>::Load((SIZE_T)Index, Ar, Variant);
	}
	else
	{
		uint8 Index = (uint8)Variant.GetIndex();
		Ar << Index;
		Visit([&Ar](auto& StoredValue)
		{
			Ar << StoredValue;
		}, Variant);
	}
	return Ar;
}
