// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/MemoryOps.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "Delegates/IntegerSequence.h"
#include "Concepts/Insertable.h"

#include "Misc/AssertionMacros.h"

template <typename T, typename... Ts>
class TVariant;

/** Determine if a type is a variant */
template <typename T>
constexpr bool TIsVariant_V = false;

template <typename... Ts>
constexpr bool TIsVariant_V<TVariant<Ts...>> = true;

template <typename T> constexpr bool TIsVariant_V<const          T> = TIsVariant_V<T>;
template <typename T> constexpr bool TIsVariant_V<      volatile T> = TIsVariant_V<T>;
template <typename T> constexpr bool TIsVariant_V<const volatile T> = TIsVariant_V<T>;

/** Determine the number of types in a TVariant */
template <typename>
constexpr SIZE_T TVariantSize_V = 0;

template <typename... Ts>
constexpr SIZE_T TVariantSize_V<TVariant<Ts...>> = sizeof...(Ts);

template <typename T> constexpr SIZE_T TVariantSize_V<const          T> = TVariantSize_V<T>;
template <typename T> constexpr SIZE_T TVariantSize_V<      volatile T> = TVariantSize_V<T>;
template <typename T> constexpr SIZE_T TVariantSize_V<const volatile T> = TVariantSize_V<T>;


namespace UE
{
namespace Core
{
namespace Private
{
	/** A shim to get at FArchive through a dependent name, allowing TVariant.h to not include Archive.h. Only calling code that needs serialization has to include it. */
	template <typename T>
	struct TAlwaysFArchive
	{
		using Type = FArchive;
	};

	/** Determine if all the types in a template parameter pack has duplicate types */
	template <typename...>
	struct TTypePackContainsDuplicates;

	/** A template parameter pack containing a single type has no duplicates */
	template <typename T>
	struct TTypePackContainsDuplicates<T>
	{
		static constexpr inline bool Value = false;
	};

	/**
	 * A template parameter pack containing the same type adjacently contains duplicate types.
	 * The next structure ensures that we check all pairs of types in a template parameter pack.
	 */
	template <typename T, typename... Ts>
	struct TTypePackContainsDuplicates<T, T, Ts...>
	{
		static constexpr inline bool Value = true;
	};

	/** Check all pairs of types in a template parameter pack to determine if any type is duplicated */
	template <typename T, typename U, typename... Rest>
	struct TTypePackContainsDuplicates<T, U, Rest...>
	{
		static constexpr inline bool Value = TTypePackContainsDuplicates<T, Rest...>::Value || TTypePackContainsDuplicates<U, Rest...>::Value;
	};

	/** Determine if any of the types in a template parameter pack are references */
	template <typename... Ts>
	struct TContainsReferenceType
	{
		static constexpr inline bool Value = (std::is_reference_v<Ts> || ...);
	};

	/** Determine the max alignof and sizeof of all types in a template parameter pack and provide a type that is compatible with those sizes */
	template <typename... Ts>
	struct TVariantStorage
	{
		static constexpr SIZE_T MaxOf(const SIZE_T Sizes[])
		{
			SIZE_T MaxSize = 0;
			for (SIZE_T Itr = 0; Itr < sizeof...(Ts); ++Itr)
			{
				if (Sizes[Itr] > MaxSize)
				{
					MaxSize = Sizes[Itr];
				}
			}
			return MaxSize;
		}
		static constexpr SIZE_T MaxSizeof()
		{
			constexpr SIZE_T Sizes[] = { sizeof(Ts)... };
			return MaxOf(Sizes);
		}
		static constexpr SIZE_T MaxAlignof()
		{
			constexpr SIZE_T Sizes[] = { alignof(Ts)... };
			return MaxOf(Sizes);
		}

		static constexpr inline SIZE_T SizeofValue = MaxSizeof();
		static constexpr inline SIZE_T AlignofValue = MaxAlignof();
		static_assert(SizeofValue > 0, "MaxSizeof must be greater than 0");
		static_assert(AlignofValue > 0, "MaxAlignof must be greater than 0");

		/** Interpret the underlying data as the type in the variant parameter pack at the compile-time index. This function is used to implement Visit and should not be used directly */
		template <SIZE_T N>
		auto& GetValueAsIndexedType() &
		{
			using ReturnType = typename TNthTypeFromParameterPack<N, Ts...>::Type;
			return *reinterpret_cast<ReturnType*>(&Storage);
		}

		/** Interpret the underlying data as the type in the variant parameter pack at the compile-time index. This function is used to implement Visit and should not be used directly */
		template <SIZE_T N>
		auto&& GetValueAsIndexedType() &&
		{
			using ReturnType = typename TNthTypeFromParameterPack<N, Ts...>::Type;
			return (ReturnType&&)GetValueAsIndexedType<N>();
		}

		/** Interpret the underlying data as the type in the variant parameter pack at the compile-time index. This function is used to implement Visit and should not be used directly */
		template <SIZE_T N>
		const auto& GetValueAsIndexedType() const&
		{
			// Temporarily remove the const qualifier so we can implement GetValueAsIndexedType in one location.
			return const_cast<TVariantStorage*>(this)->template GetValueAsIndexedType<N>();
		}

		TAlignedBytes<SizeofValue, AlignofValue> Storage;
	};

	/** Helper to lookup indices of each type in a template parameter pack */
	template <SIZE_T N, typename LookupType, typename... Ts>
	struct TParameterPackTypeIndexHelper
	{
		static constexpr inline SIZE_T Value = (SIZE_T)-1;
	};

	/** When the type we're looking up bubbles up to the top, we return the current index */
	template <SIZE_T N, typename T, typename... Ts>
	struct TParameterPackTypeIndexHelper<N, T, T, Ts...>
	{
		static constexpr inline SIZE_T Value = N;
	};

	/** When different type than the lookup is at the front of the parameter pack, we increase the index and move to the next type */
	template <SIZE_T N, typename LookupType, typename T, typename... Ts>
	struct TParameterPackTypeIndexHelper<N, LookupType, T, Ts...>
	{
		static constexpr inline SIZE_T Value = TParameterPackTypeIndexHelper<N + 1, LookupType, Ts...>::Value;
	};

	/** Entry-point for looking up the index of a type in a template parameter pack */
	template <typename LookupType, typename... Ts>
	struct TParameterPackTypeIndex
	{
		static constexpr inline SIZE_T Value = TParameterPackTypeIndexHelper<0, LookupType, Ts...>::Value;
	};

	/** An adapter for calling DestructItem */
	template <typename T>
	struct TDestructorCaller
	{
		static constexpr void Destruct(void* Storage)
		{
			DestructItem(static_cast<T*>(Storage));
		}
	};

	/** Lookup a type in a template parameter pack by its index and call the destructor */
	template <typename... Ts>
	struct TDestructorLookup
	{
		/** If the index matches, call the destructor, otherwise call with the next index and type in the parameter pack*/
		static void Destruct(SIZE_T TypeIndex, void* Value)
		{
			static constexpr void(*Destructors[])(void*) = { &TDestructorCaller<Ts>::Destruct... };
			check(TypeIndex < UE_ARRAY_COUNT(Destructors));
			Destructors[TypeIndex](Value);
		}
	};

	/** An adapter for calling a copy constructor of a type */
	template <typename T>
	struct TCopyConstructorCaller
	{
		/** Call the copy constructor of a type with the provided memory location and value */
		static void Construct(void* Storage, const void* Value)
		{
			::new(Storage) T(*static_cast<const T*>(Value));
		}
	};

	/** A utility for calling a type's copy constructor based on an index into a template parameter pack */
	template <typename... Ts>
	struct TCopyConstructorLookup
	{
		/** Construct the type at the index in the template parameter pack with the provided memory location and value */
		static void Construct(SIZE_T TypeIndex, void* Storage, const void* Value)
		{
			static constexpr void(*CopyConstructors[])(void*, const void*) = { &TCopyConstructorCaller<Ts>::Construct... };
			check(TypeIndex < UE_ARRAY_COUNT(CopyConstructors));
			CopyConstructors[TypeIndex](Storage, Value);
		}
	};


	/** A utility for calling a type's move constructor based on an index into a template parameter pack */
	template <typename T>
	struct TMoveConstructorCaller
	{
		/** Call the move constructor of a type with the provided memory location and value */
		static void Construct(void* Storage, void* Value)
		{
			::new(Storage) T(MoveTemp(*static_cast<T*>(Value)));
		}
	};

	/** A utility for calling a type's move constructor based on an index into a template parameter pack */
	template <typename... Ts>
	struct TMoveConstructorLookup
	{
		/** Construct the type at the index in the template parameter pack with the provided memory location and value */
		static void Construct(SIZE_T TypeIndex, void* Target, void* Source)
		{
			static constexpr void(*MoveConstructors[])(void*, void*) = { &TMoveConstructorCaller<Ts>::Construct... };
			check(TypeIndex < UE_ARRAY_COUNT(MoveConstructors));
			MoveConstructors[TypeIndex](Target, Source);
		}
	};

	/** A utility for loading a specific type from FArchive into a TVariant */
	template <typename T, typename VariantType>
	struct TVariantLoadFromArchiveCaller
	{
		/** Default construct the type and load it from the FArchive */
		static void Load(FArchive& Ar, VariantType& OutVariant)
		{
			OutVariant.template Emplace<T>();
			Ar << OutVariant.template Get<T>();
		}
	};

	/** A utility for loading a type from FArchive based on an index into a template parameter pack. */
	template <typename... Ts>
	struct TVariantLoadFromArchiveLookup
	{
		using VariantType = TVariant<Ts...>;
		static_assert((std::is_default_constructible_v<Ts> && ...), "Each type in TVariant template parameter pack must be default constructible in order to use FArchive serialization");
		static_assert((UE::CInsertable<FArchive&, Ts> && ...), "Each type in TVariant template parameter pack must be able to use operator<< with an FArchive");

		/** Load the type at the specified index from the FArchive and emplace it into the TVariant */
		static void Load(SIZE_T TypeIndex, FArchive& Ar, VariantType& OutVariant)
		{
			static constexpr void(*Loaders[])(FArchive&, VariantType&) = { &TVariantLoadFromArchiveCaller<Ts, VariantType>::Load... };
			check(TypeIndex < UE_ARRAY_COUNT(Loaders));
			Loaders[TypeIndex](Ar, OutVariant);
		}
	};

	/** Determine if the type with the provided index in the template parameter pack is the same */
	template <typename LookupType, typename... Ts>
	struct TIsType
	{
		/** Check if the type at the provided index is the lookup type */
		static bool IsSame(SIZE_T TypeIndex)
		{
			static constexpr bool bIsSameType[] = { std::is_same_v<Ts, LookupType>... };
			check(TypeIndex < UE_ARRAY_COUNT(bIsSameType));
			return bIsSameType[TypeIndex];
		}
	};

	/** Encode the stored index of a bunch of variants into a single value used to lookup a Visit invocation function */
	template <typename T>
	inline SIZE_T EncodeIndices(const T& Variant)
	{
		return Variant.GetIndex();
	}

	template <typename Variant0, typename... Variants>
	inline SIZE_T EncodeIndices(const Variant0& First, const Variants&... Rest)
	{
		return First.GetIndex() + TVariantSize_V<Variant0> * EncodeIndices(Rest...);
	}

	/** Inverse operation of EncodeIndices. Decodes an encoded index into the individual index for the specified variant index. */
	constexpr SIZE_T DecodeIndex(SIZE_T EncodedIndex, SIZE_T VariantIndex, const SIZE_T* VariantSizes)
	{
		while (VariantIndex)
		{
			EncodedIndex /= *VariantSizes;
			--VariantIndex;
			++VariantSizes;
		}
		return EncodedIndex % *VariantSizes;
	}

	/** Cast a TVariant to its private base */
	template <typename... Ts>
	UE_FORCEINLINE_HINT TVariantStorage<Ts...>& CastToStorage(TVariant<Ts...>& Variant)
	{
		return *(TVariantStorage<Ts...>*)(&Variant);
	}

	template <typename... Ts>
	UE_FORCEINLINE_HINT TVariantStorage<Ts...>&& CastToStorage(TVariant<Ts...>&& Variant)
	{
		return (TVariantStorage<Ts...>&&)(*(TVariantStorage<Ts...>*)(&Variant));
	}

	template <typename... Ts>
	UE_FORCEINLINE_HINT const TVariantStorage<Ts...>& CastToStorage(const TVariant<Ts...>& Variant)
	{
		return *(const TVariantStorage<Ts...>*)(&Variant);
	}

	/** Invocation detail for a single combination of stored variant indices */
	template <SIZE_T EncodedIndex, SIZE_T... VariantIndices, typename Func, typename... Variants>
	inline decltype(auto) VisitApplyEncoded(Func&& Callable, Variants&&... Args)
	{
		constexpr SIZE_T VariantSizes[] = { TVariantSize_V<std::decay_t<Variants>>... };
		return Callable(CastToStorage(Forward<Variants>(Args)).template GetValueAsIndexedType<DecodeIndex(EncodedIndex, VariantIndices, VariantSizes)>()...);
	}

	/**
	 * Work around used to separate pack expansion of EncodedIndices and VariantIndices in VisitImpl below when defining the Invokers array.
	 *
	 * Ideally the line below would only need to be written as:
	 * constexpr InvokeFn Invokers[] = { &VisitApplyEncoded<EncodedIndices, VariantIndices...>... };
	 *
	 * Due to what appears to be a lexing bug, MSVC 2017 tries to expand EncodedIndices and VariantIndices together
	 */
	template <typename InvokeFn, SIZE_T... VariantIndices>
	struct TWrapper
	{
		template <SIZE_T EncodedIndex>
		static constexpr InvokeFn FuncPtr = &VisitApplyEncoded<EncodedIndex, VariantIndices...>;
	};

	/** Implementation detail for Visit(Callable, Variants...). Builds an array of invokers, and forwards the variants to the callable for the specific EncodedIndex */
	template <typename Func, SIZE_T... EncodedIndices, SIZE_T... VariantIndices, typename... Variants>
	decltype(auto) VisitImpl(SIZE_T EncodedIndex, Func&& Callable, TIntegerSequence<SIZE_T, EncodedIndices...>&&, TIntegerSequence<SIZE_T, VariantIndices...>&& VariantIndicesSeq, Variants&&... Args)
	{
		using ReturnType = decltype(VisitApplyEncoded<0, VariantIndices...>(Forward<Func>(Callable), Forward<Variants>(Args)...));
		using InvokeFn = ReturnType(*)(Func&&, Variants&&...);
		using WrapperType = TWrapper<InvokeFn, VariantIndices...>;
		static constexpr InvokeFn Invokers[] = { WrapperType::template FuncPtr<EncodedIndices>... };
		return Invokers[EncodedIndex](Forward<Func>(Callable), Forward<Variants>(Args)...);
	}
} // namespace Private
} // namespace Core
} // namespace UE
