// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformAtomics.h"
#include "Misc/AssertionMacros.h"
#include "Templates/RemoveReference.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Misc/IFilter.h"

/**
 *	A generic filter specialized for text restrictions
 */
template< typename ItemType >
class TTextFilter : public IFilter< ItemType >, public TSharedFromThis< TTextFilter< ItemType > >
{
public:

	/**	Defines a function signature for functions used to transform an Item into an array of FStrings */
	DECLARE_DELEGATE_TwoParams( FItemToStringArray, ItemType, OUT TArray< FString >& );

	/** Defines a function signature used to test a complex expression for an Item */
	DECLARE_DELEGATE_RetVal_FiveParams( bool, FItemTestComplexExpression, ItemType, const FName& /*InKey*/, const FTextFilterString& /*InValue*/, ETextFilterComparisonOperation /*InComparisonOperation*/, ETextFilterTextComparisonMode /*InTextComparisonMode*/ );

	/** 
	 *	TTextFilter Constructor
	 *	
	 * @param	InTransform		A required delegate used to populate an array of strings based on an Item
	 */
	TTextFilter( FItemToStringArray InTransformDelegate )
		: TextFilterExpressionContext( InTransformDelegate, FItemTestComplexExpression() )
		, TextFilterExpressionEvaluator( ETextFilterExpressionEvaluatorMode::BasicString )
	{
		check( InTransformDelegate.IsBound() );
	}

	TTextFilter( FItemToStringArray InTransformDelegate, FItemTestComplexExpression InTestComplexExpressionDelegate )
		: TextFilterExpressionContext( InTransformDelegate, InTestComplexExpressionDelegate )
		, TextFilterExpressionEvaluator( ETextFilterExpressionEvaluatorMode::Complex )
	{
		check( InTransformDelegate.IsBound() );
		check( InTestComplexExpressionDelegate.IsBound() );
	}

	DECLARE_DERIVED_EVENT(TTextFilter, IFilter<ItemType>::FChangedEvent, FChangedEvent);
	virtual FChangedEvent& OnChanged() override { return ChangedEvent; }

	/** 
	 * Returns whether the specified Item passes the Filter's text restrictions 
	 *
	 *	@param	InItem	The Item to check 
	 *	@return			Whether the specified Item passed the filter
	 */
	virtual bool PassesFilter( ItemType InItem ) const override
	{
		if (TextFilterExpressionEvaluator.GetFilterType() == ETextFilterExpressionType::Empty)
		{
			return true;
		}
		
		TextFilterExpressionContext.SetItem(&InItem);
		const bool bResult = TextFilterExpressionEvaluator.TestTextFilter(TextFilterExpressionContext);
		TextFilterExpressionContext.ClearItem();
		return bResult;
	}

	/** Returns the unsanitized and unsplit filter terms */
	FText GetRawFilterText() const
	{
		return TextFilterExpressionEvaluator.GetFilterText();
	}

	/** Set the Text to be used as the Filter's restrictions */
	void SetRawFilterText( const FText& InFilterText )
	{
		if (TextFilterExpressionEvaluator.SetFilterText(InFilterText))
		{
			ChangedEvent.Broadcast();
		}
	}

	/** Get the last error returned from lexing or compiling the current filter text */
	FText GetFilterErrorText() const
	{
		return TextFilterExpressionEvaluator.GetFilterErrorText();
	}

private:

	class FTextFilterExpressionContext : public ITextFilterExpressionContext
	{
	public:
		typedef typename TRemoveReference<ItemType>::Type* ItemTypePtr;

		FTextFilterExpressionContext(const FItemToStringArray& InTransformArrayDelegate, const FItemTestComplexExpression& InTestComplexExpressionDelegate)
			: TransformArrayDelegate(InTransformArrayDelegate)
			, TestComplexExpressionDelegate(InTestComplexExpressionDelegate)
			, ItemPtr(nullptr)
		{
		}

		void SetItem(ItemTypePtr InItem)
		{
			ItemPtr = InItem;
			TransformArrayDelegate.Execute(*ItemPtr, ItemBasicStrings);
		}

		void ClearItem()
		{
			ItemPtr = nullptr;
			ItemBasicStrings.Reset();
		}

		virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override
		{
			for (const FString& BasicString : ItemBasicStrings)
			{
				if (TextFilterUtils::TestBasicStringExpression(BasicString, InValue, InTextComparisonMode))
				{
					return true;
				}
			}
			return false;
		}

		virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override
		{
			if (ItemPtr && TestComplexExpressionDelegate.IsBound())
			{
				return TestComplexExpressionDelegate.Execute(*ItemPtr, InKey, InValue, InComparisonOperation, InTextComparisonMode);
			}
			return false;
		}

	private:
		/** The delegate used to transform an item into an array of strings */
		FItemToStringArray TransformArrayDelegate;

		/** The delegate used to test a complex expression for an item */
		FItemTestComplexExpression TestComplexExpressionDelegate;

		/** Pointer to the item we're currently filtering */
		ItemTypePtr ItemPtr;

		/** The strings extracted from the item we're currently filtering */
		TArray<FString> ItemBasicStrings;
	};

	/** Transient context data, used when calling PassesFilter. Kept around to minimize re-allocations between multiple calls to PassesFilter */
	mutable FTextFilterExpressionContext TextFilterExpressionContext;

	/** Expression evaluator that can be used to perform complex text filter queries */
	FTextFilterExpressionEvaluator TextFilterExpressionEvaluator;

	/**	The event that fires whenever new search terms are provided */
	FChangedEvent ChangedEvent;
};
