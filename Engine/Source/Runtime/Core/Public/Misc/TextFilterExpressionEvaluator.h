// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Misc/ExpressionParser.h"
#include "Misc/ExpressionParserTypes.h"
#include "Misc/Optional.h"
#include "Misc/TextFilterUtils.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

class FTextToken;

/** Defines the complexity of the current filter terms. Complex in this case means that the expression will perform key->value comparisons */
enum class ETextFilterExpressionType : uint8
{
	Invalid,
	Empty,
	BasicString,
	Complex,
};

/** Defines whether or not the expression parser can evaluate complex expressions */
enum class ETextFilterExpressionEvaluatorMode : uint8
{
	BasicString,
	Complex,
};

/** Interface to implement to allow FTextFilterExpressionEvaluator to perform its comparison tests in TestTextFilter */
class ITextFilterExpressionContext
{
public:
	/** Test the given value against the strings extracted from the current item */
	virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const = 0;

	/** Perform a complex expression test for the current item */
	virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const = 0;

protected:
	virtual ~ITextFilterExpressionContext() = default;
};

// A simple basic string expression context for FString
class FBasicStringFilterExpressionContext : public ITextFilterExpressionContext
{
public:
	explicit FBasicStringFilterExpressionContext(FString&& InTestString)
		: TestString(InTestString)
	{
	}
	explicit FBasicStringFilterExpressionContext(const FString& InTestString)
		: TestString(InTestString)
	{
	}

	virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		return TextFilterUtils::TestBasicStringExpression(TestString, InValue, InTextComparisonMode);
	}

	virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		return false;
	}

private:
	const FString TestString;
};

namespace TextFilterExpressionParser
{
	class FTextToken
	{
	public:
		enum class EInvertResult : uint8
		{
			No,
			Yes,
		};

		FTextToken(FTextFilterString InString, ETextFilterTextComparisonMode InTextComparisonMode, const EInvertResult InInvertResult)
			: String(MoveTemp(InString))
			, TextComparisonMode(MoveTemp(InTextComparisonMode))
			, InvertResult(InInvertResult)
		{
		}

		FTextToken(const FTextToken& Other)
			: String(Other.String)
			, TextComparisonMode(Other.TextComparisonMode)
			, InvertResult(Other.InvertResult)
		{
		}

		FTextToken(FTextToken&& Other)
			: String(MoveTemp(Other.String))
			, TextComparisonMode(Other.TextComparisonMode)
			, InvertResult(Other.InvertResult)
		{
		}

		FTextToken& operator=(const FTextToken& Other)
		{
			String = Other.String;
			TextComparisonMode = Other.TextComparisonMode;
			InvertResult = Other.InvertResult;
			return *this;
		}

		FTextToken& operator=(FTextToken&& Other)
		{
			String = MoveTemp(Other.String);
			TextComparisonMode = Other.TextComparisonMode;
			InvertResult = Other.InvertResult;
			return *this;
		}

		const FTextFilterString& GetString() const
		{
			return String;
		}

		bool EvaluateAsBasicStringExpression(const ITextFilterExpressionContext* InContext) const
		{
			if (String.IsEmpty())
			{
				// An empty string is always considered to be true
				return true;
			}

			const bool bMatched = InContext->TestBasicStringExpression(String, TextComparisonMode);
			return (InvertResult == EInvertResult::No) ? bMatched : !bMatched;
		}

		bool EvaluateAsComplexExpression(const ITextFilterExpressionContext* InContext, const FTextFilterString& InKey, const ETextFilterComparisonOperation InComparisonOperation) const
		{
			const bool bMatched = InContext->TestComplexExpression(InKey.AsName(), String, InComparisonOperation, TextComparisonMode);
			return (InvertResult == EInvertResult::No) ? bMatched : !bMatched;
		}

		ETextFilterTextComparisonMode GetTextComparisonMode() const
		{
			return TextComparisonMode;
		}

	private:
		FTextFilterString String;
		ETextFilterTextComparisonMode TextComparisonMode;
		EInvertResult InvertResult;
	};

	class FFunction
	{

	};
} // namespace TextFilterExpressionParser

#define DEFINE_TEXT_EXPRESSION_OPERATOR_NODE(TYPE, MONIKER_COUNT, ...)	\
	namespace TextFilterExpressionParser								\
	{																	\
		struct TYPE														\
		{																\
			static constexpr int32 MonikerCount = MONIKER_COUNT;			\
			static const TCHAR* Monikers[MonikerCount];					\
		};																\
	}																	\
DEFINE_EXPRESSION_NODE_TYPE(TextFilterExpressionParser::TYPE, __VA_ARGS__)

DEFINE_TEXT_EXPRESSION_OPERATOR_NODE(FSubExpressionStart,		1,	0x0E7E1BC9, 0xF0B94D5D, 0x80F44D45, 0x85A082AA)
DEFINE_TEXT_EXPRESSION_OPERATOR_NODE(FSubExpressionEnd,			1,	0x5E10956D, 0x2E17411F, 0xB6469E22, 0xB5E56E6C)

DEFINE_TEXT_EXPRESSION_OPERATOR_NODE(FEqual,					3,	0x32457EFC, 0x4928406F, 0xBD78D943, 0x633797D1)
DEFINE_TEXT_EXPRESSION_OPERATOR_NODE(FNotEqual,					2,	0x8F445A19, 0xF33443D9, 0x90D6DC85, 0xB0C5D071)
DEFINE_TEXT_EXPRESSION_OPERATOR_NODE(FLess,						1,	0xB85E222B, 0x47E24E1F, 0xBC5A384D, 0x2FF471E1)
DEFINE_TEXT_EXPRESSION_OPERATOR_NODE(FLessOrEqual,				2,	0x8C0A46B0, 0x8DAA4E2B, 0xA7FE4A23, 0xEF215918)
DEFINE_TEXT_EXPRESSION_OPERATOR_NODE(FGreater,					1,	0x6A6247F4, 0xFD78467F, 0xA6AC1244, 0x0A31E0A5)
DEFINE_TEXT_EXPRESSION_OPERATOR_NODE(FGreaterOrEqual,			2,	0x09D75C5E, 0xA29A4194, 0x8E8E5278, 0xC84365FD)

DEFINE_TEXT_EXPRESSION_OPERATOR_NODE(FOr,						4,	0xF4778B51, 0xF535414D, 0x9C0EB5F2, 0x2F2B0FD4)
DEFINE_TEXT_EXPRESSION_OPERATOR_NODE(FAnd,						4,	0x7511397A, 0x02D24DC2, 0x86729800, 0xF454C320)
DEFINE_TEXT_EXPRESSION_OPERATOR_NODE(FNot,						3,	0x03D78990, 0x41D04E26, 0x8E98AD2F, 0x74667868)

DEFINE_EXPRESSION_NODE_TYPE(TextFilterExpressionParser::FTextToken,	0x09E49538, 0x633545E3, 0x84B5644F, 0x1F11628F);
DEFINE_EXPRESSION_NODE_TYPE(TextFilterExpressionParser::FFunction,	0x084E6214, 0x032FFA48, 0x9245ABF1, 0x9E248F1A);

#undef DEFINE_TEXT_EXPRESSION_OPERATOR_NODE

DECLARE_DELEGATE_RetVal_OneParam(bool, FTokenFunctionHandler, const FTextFilterString&);

/** Defines an expression evaluator that can be used to perform complex text filter queries */
class FTextFilterExpressionEvaluator
{
public:
	/** Construction and assignment */
	CORE_API FTextFilterExpressionEvaluator(const ETextFilterExpressionEvaluatorMode InMode);
	CORE_API FTextFilterExpressionEvaluator(const FTextFilterExpressionEvaluator& Other);
	CORE_API FTextFilterExpressionEvaluator& operator=(const FTextFilterExpressionEvaluator& Other);
	CORE_API virtual ~FTextFilterExpressionEvaluator();

	/** Get the complexity of the current filter terms */
	CORE_API ETextFilterExpressionType GetFilterType() const;

	/** Get the filter terms that we're currently using */
	CORE_API FText GetFilterText() const;

	/** Set the filter terms to be compiled for evaluation later. Returns true if the filter was actually changed */
	CORE_API bool SetFilterText(const FText& InFilterText);

	/** Get the last error returned from lexing or compiling the current filter text */
	CORE_API FText GetFilterErrorText() const;

	/** Get the filter expression tokens created from the current filter text */
	CORE_API const TArray<FExpressionToken>& GetFilterExpressionTokens() const;

	/** Test our compiled filter using the given context */
	CORE_API bool TestTextFilter(const ITextFilterExpressionContext& InContext) const;

	/** Helper function to add callbacks for function tokens */
	CORE_API void AddFunctionTokenCallback(FString InFunctionName, FTokenFunctionHandler InCallback);

protected:
	/** Sets up grammar used for evaluation */
	CORE_API void SetupGrammar();

	/** Common function to construct the expression parser */
	CORE_API virtual void ConstructExpressionParser();

	/** Evaluate the given compiled result, and optionally populate OutErrorText with any error information */
	CORE_API virtual bool EvaluateCompiledExpression(const ExpressionParser::CompileResultType& InCompiledResult, const ITextFilterExpressionContext& InContext, FText* OutErrorText) const;

	/** Defines whether or not the expression parser can evaluate complex expressions */
	ETextFilterExpressionEvaluatorMode ExpressionEvaluatorMode;

	/** The cached complexity of the current filter terms */
	ETextFilterExpressionType FilterType;

	/** The the filter terms that we're currently using (compiled into CompiledFilter) */
	FText FilterText;

	/** The last error returned from lexing or compiling the current filter text */
	FText FilterErrorText;

	/** The filter expression tokens created from the current filter text (lexed from FilterText) */
	TArray<FExpressionToken> ExpressionTokens;

	/** The compiled filter created from the current filter text (compiled from FilterText) */
	TOptional<ExpressionParser::CompileResultType> CompiledFilter;

	/** If there is only one token that is basic string it will be stored here to avoid recompile per item compared. */
	TOptional<TextFilterExpressionParser::FTextToken> CompiledFilterSingleBasicStringTextToken;

	/** Mapping of function names to their callbacks */
	TMap<FString, FTokenFunctionHandler> TokenFunctionHandlers;

	/** Expression parser */
	FTokenDefinitions TokenDefinitions;
	FExpressionGrammar Grammar;
	TOperatorJumpTable<ITextFilterExpressionContext> JumpTable;
};
