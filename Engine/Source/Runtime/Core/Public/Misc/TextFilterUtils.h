// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "UObject/NameTypes.h"

/** Defines the comparison operators that can be used for a complex (key->value) comparison */
enum class ETextFilterComparisonOperation : uint8
{
	Equal,
	NotEqual,
	Less,
	LessOrEqual,
	Greater,
	GreaterOrEqual,
};

/** Defines the different ways that a string can be compared while evaluating the expression */
enum class ETextFilterTextComparisonMode : uint8
{
	Exact,
	Partial,
	StartsWith,
	EndsWith,
};

enum {NAME_WITH_NUMBER_SIZE	= NAME_SIZE + 16};

/**
 * String used by the text filter.
 * The given string will be stored as uppercase since filter text always performs case-insensitive string comparisons, so this will minimize ToUpper calls.
 */
class FTextFilterString
{
public:
	/** Default constructor */
	CORE_API FTextFilterString();

	/** Move and copy constructors */
	CORE_API FTextFilterString(const FTextFilterString& Other);
	CORE_API FTextFilterString(FTextFilterString&& Other);

	/** Construct from a string */
	CORE_API FTextFilterString(const FString& InString);
	CORE_API FTextFilterString(FString&& InString);
	CORE_API FTextFilterString(FStringView InString);
	CORE_API FTextFilterString(const TCHAR* InString);

	/** Construct from a name */
	CORE_API FTextFilterString(FName InName);

	/** Move and copy assignment */
	CORE_API FTextFilterString& operator=(const FTextFilterString& Other);
	CORE_API FTextFilterString& operator=(FTextFilterString&& Other);

	CORE_API FTextFilterString& operator=(const FString& InString);
	CORE_API FTextFilterString& operator=(FStringView InString);
	CORE_API FTextFilterString& operator=(const TCHAR* InString);
	CORE_API FTextFilterString& operator=(FName InName);

	/** Compare this string against the other, using the text comparison mode provided */
	CORE_API bool CompareText(const FTextFilterString& InOther, const ETextFilterTextComparisonMode InTextComparisonMode) const;

	/** Compare this string against the other FString, using the text comparison mode provided */
	CORE_API bool CompareFString(const FString& InOther, const ETextFilterTextComparisonMode InTextComparisonMode) const;

	/** Compare this string against a string view which has previously been converted to uppercase */
	CORE_API bool CompareFStringView(FStringView InOther, const ETextFilterTextComparisonMode InTextComparisonMode) const;

	/** Compare this string against the other FName, using the text comparison mode provided */
	CORE_API bool CompareName(const FName& InOther, const ETextFilterTextComparisonMode InTextComparisonMode) const;

	/** Are the two given strings able to be compared numberically? */
	CORE_API bool CanCompareNumeric(const FTextFilterString& InOther) const;

	/** Compare this string against the other, converting them to numbers and using the comparison operator provided - you should have tested CanCompareNumeric first! */
	CORE_API bool CompareNumeric(const FTextFilterString& InOther, const ETextFilterComparisonOperation InComparisonOperation) const;

	/** Get the internal uppercase string of this filter string */
	UE_FORCEINLINE_HINT const FString& AsString() const
	{
		return InternalString;
	}

	/** Get the internal uppercase string of this filter string as an FName */
	UE_FORCEINLINE_HINT FName AsName() const
	{
		return FName(*InternalString);
	}

	/** Is the internal string empty? */
	UE_FORCEINLINE_HINT bool IsEmpty() const
	{
		return InternalString.IsEmpty();
	}

private:
	/** Inline convert our internal string to uppercase */
	CORE_API void UppercaseInternalString();

	/** The uppercase string to use for comparisons */
	FString InternalString;

	/** The uppercase ANSI version of string to use for comparisons */
	TArray<ANSICHAR> InternalStringAnsi;
};

namespace TextFilterUtils
{
	template <typename CharType>
	void IntToStringBuffer(CharType* Dest, int32 Source, int32 MaxLen)
	{
		int64 Num						= Source; // This avoids having to deal with negating -MAX_int32-1
		bool bIsNumberNegative			= false;
		const int32 TempBufferSize		= 16; // 16 is big enough
		CharType TempNum[TempBufferSize];
		int32 TempAt					= TempBufferSize; // fill the temp string from the top down.

		// Correctly handle negative numbers and convert to positive integer.
		if( Num < 0 )
		{
			bIsNumberNegative = true;
			Num = -Num;
		}

		TempNum[--TempAt] = 0; // NULL terminator

		// Convert to string assuming base ten and a positive integer.
		do 
		{
			TempNum[--TempAt] = CharType('0') + (Num % 10);
			Num /= 10;
		} while( Num );

		// Append sign as we're going to reverse string afterwards.
		if( bIsNumberNegative )
		{
			TempNum[--TempAt] = CharType('-');
		}

		const CharType* CharPtr = TempNum + TempAt;
		const int32 NumChars = TempBufferSize - TempAt - 1;
		if (NumChars < MaxLen)
		{
			TCString<CharType>::Strncpy(Dest, CharPtr, NumChars + 1);
		}
		else if (MaxLen > 0)
		{
			*Dest = 0;
		}
	}

	/*
	* Fills string buffer with FName including number without forcing converting to wide character
	*/
	struct FNameBufferWithNumber
	{
	public:

		FNameBufferWithNumber() : bIsWide(false)
		{
		}

		FNameBufferWithNumber(const FName& Name) : bIsWide(false)
		{
			Init(Name.GetDisplayNameEntry(), Name.GetNumber());
		}

		FNameBufferWithNumber(const FNameEntry* const NameEntry, int32 NumberInternal) : bIsWide(false)
		{
			Init(NameEntry, NumberInternal);
		}

		UE_FORCEINLINE_HINT void Init(const FName& Name)
		{
			Init(Name.GetDisplayNameEntry(), Name.GetNumber());
		}

		inline void Init(const FNameEntry* const NameEntry, int32 NumberInternal)
		{
			if (NameEntry == nullptr)
			{
				bIsWide = true;
				FCString::Strncpy(WideName, TEXT("*INVALID*"), NAME_SIZE);
				return;
			}

			if (NameEntry->IsWide())
			{
				bIsWide = true;
				NameEntry->GetWideName(_WideName);
				if (NumberInternal != NAME_NO_NUMBER_INTERNAL)
				{
					int32 Len = FCStringWide::Strlen(WideName);
					WideName[Len++] = TEXT('_');
					IntToStringBuffer(WideName + Len, NAME_INTERNAL_TO_EXTERNAL(NumberInternal), 16);
				}
			}
			else
			{
				bIsWide = false;
				NameEntry->GetAnsiName(_AnsiName);
				if (NumberInternal != NAME_NO_NUMBER_INTERNAL)
				{
					int32 Len = FCStringAnsi::Strlen(AnsiName);
					AnsiName[Len++] = '_';
					IntToStringBuffer(AnsiName + Len, NAME_INTERNAL_TO_EXTERNAL(NumberInternal), 16);
				}
			}
		}

		UE_FORCEINLINE_HINT bool IsWide() const
		{
			return bIsWide;
		}

		inline ANSICHAR* GetAnsiNamePtr()
		{
			checkSlow(!IsWide());
			return AnsiName;
		}

		inline WIDECHAR* GetWideNamePtr()
		{
			checkSlow(IsWide());
			return WideName;
		}

		UE_FORCEINLINE_HINT int32 GetMaxBufferLength()
		{
			return NAME_WITH_NUMBER_SIZE;
		}

	private:

		bool bIsWide;

		union
		{
			ANSICHAR AnsiName[NAME_WITH_NUMBER_SIZE];
			WIDECHAR WideName[NAME_WITH_NUMBER_SIZE];
			ANSICHAR _AnsiName[NAME_SIZE];
			WIDECHAR _WideName[NAME_SIZE];
		};
	};

	/** Convert a wide string to ansi if all characters are ansi */
	CORE_API bool TryConvertWideToAnsi(const FString& SourceWideString, TArray<ANSICHAR>& DestAnsiString);

	/** Compare FNameEntry vs Wide or Ansi string */
	CORE_API int32 NameStrincmp(const FName& Name, const FString& WideOther, const TArray<ANSICHAR>& AnsiOther, int32 Length);

	/** Utility function to perform a basic string test for the given values */
	CORE_API bool TestBasicStringExpression(const FTextFilterString& InValue1, const FTextFilterString& InValue2, const ETextFilterTextComparisonMode InTextComparisonMode);

	/** Utility function to perform a complex expression test for the given values */
	CORE_API bool TestComplexExpression(const FTextFilterString& InValue1, const FTextFilterString& InValue2, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode);
}
