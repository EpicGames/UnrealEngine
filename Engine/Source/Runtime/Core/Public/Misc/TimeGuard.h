// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformTime.h"
#include "Misc/Build.h"
#include "Misc/DateTime.h"

#ifndef DO_TIMEGUARD
	// By default we are disabled, DO_TIMEGUARD can be set in XXX.Target.cs if so desired%
	#define DO_TIMEGUARD 0 
#endif

#ifndef DO_LIGHTWEIGHT_TIMEGUARD 
  // By default, enable lightweight timeguards if logging is enabled, except on servers
	#define DO_LIGHTWEIGHT_TIMEGUARD ( WITH_ENGINE && !UE_SERVER && !NO_LOGGING && !WITH_EDITOR )
#endif

#if DO_TIMEGUARD

DECLARE_DELEGATE_RetVal(const FString, FTimerNameDelegate);

class FTimeGuard
{

public:

	struct FGuardInfo
	{
		int			Count;
		float		Min;
		float		Max;
		float		Total;
		FDateTime	FirstTime;
		FDateTime	LastTime;

		FGuardInfo() :
			Count(0)
			, Min(FLT_MAX)
			, Max(-FLT_MAX)
			, Total(0)
		{}
	};

public:


	inline FTimeGuard(FTimerNameDelegate InNameDelegate, const float InTargetMS = 0.0)
		: Name(nullptr), ObjectName(NAME_None)
	{
		NameDelegate = (bEnabled && IsInGameThread()) ? InNameDelegate : nullptr;

		if (NameDelegate.IsBound())
		{
			TargetTimeMS = (InTargetMS > 0) ? InTargetMS : FrameTimeThresholdMS;
			StartTime = FPlatformTime::Seconds();
		}
	}

	inline FTimeGuard(TCHAR const* InName, FName InObjectName = NAME_None, const float InTargetMS = 0.0)
		: Name(nullptr), ObjectName(NAME_None)
	{
		Name = (bEnabled && IsInGameThread()) ? InName : nullptr;

		if (Name)
		{
			ObjectName = InObjectName;
			TargetTimeMS = (InTargetMS > 0) ? InTargetMS : FrameTimeThresholdMS;
			StartTime = FPlatformTime::Seconds();
		}
	}

	/**
	* Updates the stat with the time spent
	*/
	inline ~FTimeGuard()
	{
		if (Name)
		{
			double delta = (FPlatformTime::Seconds() - StartTime) * 1000;

			if (delta > TargetTimeMS)
			{
				if ( ObjectName != NAME_None )
				{
					ReportHitch( *FString::Printf(TEXT("%s %s"), Name, *ObjectName.ToString()), delta, true );
				}
				else
				{
					ReportHitch(Name, delta, false);
				}
			}
		}
		else if (NameDelegate.IsBound())
		{
			double delta = (FPlatformTime::Seconds() - StartTime) * 1000;

			if (delta > TargetTimeMS)
			{
				FString Temp = NameDelegate.Execute();
				ReportHitch(*Temp, delta, true);
			}
		}
	}

	// Can be used externally to extract / clear data
	static CORE_API void						ClearData();
	static CORE_API void						SetEnabled(bool InEnable);
	static CORE_API void						SetFrameTimeThresholdMS(float InTimeMS);
	static CORE_API void						GetData(TMap<const TCHAR*, FGuardInfo, FDefaultSetAllocator, TStringPointerMapKeyFuncs_DEPRECATED<const TCHAR*, FGuardInfo>>& Dest);

protected:

	// Used for reporting
	static CORE_API void																											ReportHitch(const TCHAR* InName, float TimeMS, bool VolatileName );
	static TMap<const TCHAR*, FGuardInfo, FDefaultSetAllocator, TStringPointerMapKeyFuncs_DEPRECATED<const TCHAR*, FGuardInfo>>		HitchData;
	static TSet<const TCHAR *, TStringPointerSetKeyFuncs_DEPRECATED<const TCHAR*>>													VolatileNames; // any names which come in volatile we allocate them to a static string and put them in this array
	static FCriticalSection																											ReportMutex;
	static CORE_API bool																											bEnabled;
	static CORE_API float																											FrameTimeThresholdMS;

protected:

	// Per-stat tracking
	TCHAR const*	Name;
	FName			ObjectName;
	FTimerNameDelegate NameDelegate;
	float			TargetTimeMS;
	double			StartTime;
};

#define SCOPE_TIME_GUARD(name) \
	FTimeGuard ANONYMOUS_VARIABLE(TimeGuard)(name);

#define SCOPE_TIME_GUARD_MS(name, timeMs) \
	FTimeGuard ANONYMOUS_VARIABLE(TimeGuard)(name, NAME_None, timeMs);

#define SCOPE_TIME_GUARD_NAMED(name, fname) \
	FTimeGuard ANONYMOUS_VARIABLE(TimeGuard)(name, fname);

#define SCOPE_TIME_GUARD_NAMED_MS(name, fname, timems) \
	FTimeGuard ANONYMOUS_VARIABLE(TimeGuard)(name, fname, timems);




#define SCOPE_TIME_GUARD_DELEGATE(inDelegate) \
	FTimeGuard ANONYMOUS_VARIABLE(TimeGuard)(inDelegate);

#define SCOPE_TIME_GUARD_DELEGATE_MS(inDelegate, timems) \
	FTimeGuard ANONYMOUS_VARIABLE(TimeGuard)(inDelegate, timems);

#define CLEAR_TIME_GUARDS FTimeGuard::ClearData

#define ENABLE_TIME_GUARDS(bEnabled) FTimeGuard::SetEnabled(bEnabled)

#else

#define SCOPE_TIME_GUARD(name)
#define SCOPE_TIME_GUARD_MS(name, timeMs)
#define SCOPE_TIME_GUARD_NAMED(name, fname)
#define SCOPE_TIME_GUARD_NAMED_MS(name, fname, timeMs)
#define CLEAR_TIME_GUARDS()
#define ENABLE_TIME_GUARDS(bEnabled)

#endif // DO_TIMEGUARD

// Lightweight time guard, suitable for shipping builds with logging. 
// Note: Threshold of 0 disables the timeguard
#if DO_LIGHTWEIGHT_TIMEGUARD

  #define LIGHTWEIGHT_TIME_GUARD_BEGIN( Name, ThresholdMS ) \
	float UE_JOIN(__TimeGuard_ThresholdMS_, Name) = ThresholdMS; \
	uint64 UE_JOIN(__TimeGuard_StartCycles_, Name) = ( ThresholdMS > 0.0f ) ? FPlatformTime::Cycles64() : 0;

  #define LIGHTWEIGHT_TIME_GUARD_END( Name, NameStringCode ) \
	if ( UE_JOIN(__TimeGuard_ThresholdMS_, Name) > 0.0f ) \
	{\
		float UE_JOIN(__TimeGuard_MSElapsed_,Name) = FPlatformTime::ToMilliseconds64( FPlatformTime::Cycles64() - UE_JOIN(__TimeGuard_StartCycles_,Name) ); \
		if ( UE_JOIN(__TimeGuard_MSElapsed_,Name) > UE_JOIN(__TimeGuard_ThresholdMS_, Name) ) \
		{ \
			FString ReportName = NameStringCode; \
			UE_LOGF(LogCore, Warning, "LIGHTWEIGHT_TIME_GUARD: %ls - %ls took %.2fms!", TEXT(#Name), *ReportName, UE_JOIN(__TimeGuard_MSElapsed_,Name)); \
		} \
	}
#else
  #define LIGHTWEIGHT_TIME_GUARD_BEGIN( Name, ThresholdMS ) 
  #define LIGHTWEIGHT_TIME_GUARD_END( Name, NameStringCode )
#endif
