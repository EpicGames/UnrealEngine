// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "HAL/Platform.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/Timespan.h"

/**
 * A timecode that stores time in HH:MM:SS format with the remainder
 * of time represented by an integer frame count.
 */
struct FTimecode
{
	/**
	 * Default construction for UObject purposes
	 */
	FTimecode()
		: Hours(0)
		, Minutes(0)
		, Seconds(0)
		, Frames(0)
		, Subframe(0)
		, bDropFrameFormat(false)
	{}

	/**
	 * User construction from a number of hours minutes seconds and frames.
	 * @param InbDropFrame - If true, this Timecode represents a "Drop Frame Timecode" format which
							skips the first frames of every minute (except those ending in multiples of 10)
							to account for drift when using a fractional NTSC framerate.
 	 */
	explicit FTimecode(int32 InHours, int32 InMinutes, int32 InSeconds, int32 InFrames, bool InbDropFrame)
		: Hours(InHours)
		, Minutes(InMinutes)
		, Seconds(InSeconds)
		, Frames(InFrames)
		, bDropFrameFormat(InbDropFrame)
	{
		checkSlow(IsValid());
	}

	/**
	 * User construction from a number of hours minutes seconds frames, and subframes.
	 * @param InbDropFrame - If true, this Timecode represents a "Drop Frame Timecode" format which
							skips the first frames of every minute (except those ending in multiples of 10)
							to account for drift when using a fractional NTSC framerate.
 	 */
	explicit FTimecode(int32 InHours, int32 InMinutes, int32 InSeconds, int32 InFrames, float InSubframe, bool InbDropFrame)
		: Hours(InHours)
		, Minutes(InMinutes)
		, Seconds(InSeconds)
		, Frames(InFrames)
		, Subframe(InSubframe)
		, bDropFrameFormat(InbDropFrame)
	{
		checkSlow(IsValid());
	}

	/**
	 * User construction from a time in seconds
	 * @param InbDropFrame	- If true, this Timecode represents a "Drop Frame Timecode" format which
							skips the first frames of every minute (except those ending in multiples of 10)
							to account for drift when using a fractional NTSC framerate.
	 * @param InbRollover	- If true, the hours will be the modulo of 24.
	 * @note Be aware that the Cycles may not correspond to the System Time. See FDateTime and "leap seconds".
	 */
	explicit FTimecode(double InSeconds, const FFrameRate& InFrameRate, bool InbDropFrame, bool InbRollover)
	{
		if (InbRollover)
		{
			const int32 NumberOfSecondsPerDay = 60 * 60 * 24;
			double IntegralPart = 0.0;
			double Fractional = FMath::Modf(InSeconds, &IntegralPart);
			const int32 CurrentRolloverSeconds = (int32)IntegralPart % NumberOfSecondsPerDay;
			InSeconds = (double)CurrentRolloverSeconds + Fractional;
		}
		
		const int32 NumberOfFrames = (int32)FMath::RoundToDouble(InSeconds * InFrameRate.AsDecimal());
		*this = FromFrameNumber(FFrameNumber(NumberOfFrames), InFrameRate, InbDropFrame);
	}

	/**
	 * User construction from a time in seconds
	 * @param InbRollover	- If true, the hours will be the modulo of 24.
	 * @note Be aware that the Cycles may not correspond to the System Time. See FDateTime and "leap seconds".
	 */
	explicit FTimecode(double InSeconds, const FFrameRate& InFrameRate, bool InbRollover)
		: FTimecode(InSeconds, InFrameRate, UseDropFormatTimecode(InFrameRate), InbRollover)
	{
	}

	

	friend bool operator==(const FFrameRate& A, const FFrameRate& B);
	friend bool operator!=(const FFrameRate& A, const FFrameRate& B);

public:

	/**
	 * Converts this Timecode back into a Frame Number at the given framerate, taking into account if this is a drop-frame format timecode.
	 */
	FFrameNumber ToFrameNumber(const FFrameRate& InFrameRate) const
	{
		const int32 NumberOfFramesInSecond = FMath::CeilToInt((float)InFrameRate.AsDecimal());
		const int32 NumberOfFramesInMinute = NumberOfFramesInSecond * 60;
		const int32 NumberOfFramesInHour = NumberOfFramesInMinute * 60;

		if (NumberOfFramesInSecond <= 0)
		{
			return FFrameNumber();
		}

		// Do a quick pre-pass to take any overflow values and move them into bigger time units.
		int32 SafeSeconds = Seconds + Frames / NumberOfFramesInSecond;
		int32 SafeFrames = Frames % NumberOfFramesInSecond;

		int32 SafeMinutes = Minutes + SafeSeconds / 60;
		SafeSeconds = SafeSeconds % 60;

		int32 SafeHours = Hours + SafeMinutes / 60;
		SafeMinutes = SafeMinutes % 60;

		if (bDropFrameFormat)
		{
			const int32 NumberOfTimecodesToDrop = NumberOfFramesInSecond <= 30 ? 2 : 4;

			// Calculate how many minutes there are total so we can know how many times we've skipped timecodes.
			int32 TotalMinutes = (SafeHours * 60) + SafeMinutes;

			// We skip timecodes 9 times out of every 10.
			int32 TotalDroppedFrames = NumberOfTimecodesToDrop * (TotalMinutes - (int32)FMath::RoundToZero(TotalMinutes / 10.0));
			int32 TotalFrames = (SafeHours * NumberOfFramesInHour) + (SafeMinutes * NumberOfFramesInMinute) + (SafeSeconds * NumberOfFramesInSecond)
				+ SafeFrames - TotalDroppedFrames;

			return FFrameNumber(TotalFrames);
		}
		else
		{
			int32 TotalFrames = (SafeHours *NumberOfFramesInHour) + (SafeMinutes * NumberOfFramesInMinute) + (SafeSeconds * NumberOfFramesInSecond) + SafeFrames;
			return FFrameNumber(TotalFrames);
		}
	}

	/**
	 * Create a FTimecode from a specific frame number at the given frame rate. Optionally supports creating a drop frame timecode,
	 * which drops certain timecode display numbers to help account for NTSC frame rates which are fractional.
	 *
	 * @param InFrameNumber - The frame number to convert into a timecode. This should already be converted to InFrameRate's resolution.
	 * @param InFrameRate	- The framerate that this timecode is based in. This should be the playback framerate as it is used to determine
	 *						  when the Frame value wraps over.
	 * @param InbDropFrame	- If true, the returned timecode will drop the first two frames on every minute (except when Minute % 10 == 0)
	 *						  This is only valid for NTSC framerates (29.97, 59.94) and will assert if you try to create a drop-frame format
	 *						  from a non-valid framerate. All framerates can be represented when in non-drop frame format.
	 */
	static FTimecode FromFrameNumber(const FFrameNumber& InFrameNumber, const FFrameRate& InFrameRate, bool InbDropFrame)
	{
		const int32 NumberOfFramesInSecond = FMath::CeilToInt((float)InFrameRate.AsDecimal());
		const int32 NumberOfFramesInMinute = NumberOfFramesInSecond * 60;
		const int32 NumberOfFramesInHour = NumberOfFramesInMinute * 60;

		if (NumberOfFramesInSecond <= 0)
		{
			return FTimecode();
		}

		if (InbDropFrame)
		{
			// Drop Frame Timecode (DFT) was created to address the issue with playing back whole frames at fractional framerates.
			// DFT is confusingly named though, as no frame numbers are actually dropped, only the display of them. At an ideal 30fps,
			// there are 108,000 frames in an hour. When played back at 29.97 however, there are only 107,892 frames per hour. This
			// leaves us a difference of 108 frames per hour (roughly ~3.6s!). DFT works by accumulating error until the error is significant
			// enough to catch up by a frame. This is accomplished by dropping two (or four) timecode numbers every minute which gives us a
			// total difference of 2*60 = 120 frames per hour. Unfortunately 120 puts us out of sync again as the difference is only 108 frames,
			// so we need to get 12 frames back. By not dropping frames every 10th minute, that gives us 2 frames * 6 (00, 10, 20, 30, 40, 50)
			// which gets us the 12 frame difference we need. In short, we drop frames (skip timecode numbers) every minute, on the minute, except
			// when (Minute % 10 == 0)

			// 29.97 drops two timecode values (frames 0 and 1) while 59.94 drops four values (frames 0, 1, 2 and 3)
			const int32 NumberOfTimecodesToDrop = NumberOfFramesInSecond <= 30 ? 2 : 4;

			// At an ideal 30fps there would be 18,000 frames every 10 minutes, but at 29.97 there's only 17,982 frames.
			const int32 NumTrueFramesPerTenMinutes = FMath::FloorToInt((float)((60 * 10) * InFrameRate.AsDecimal()));

			// Calculate out how many times we've skipped dropping frames (ie: Minute 15 gives us a value of 1, as we've only didn't drop frames on the 10th minute)
			const int32 NumTimesSkippedDroppingFrames = FMath::Abs(InFrameNumber.Value) / NumTrueFramesPerTenMinutes;

			// Now we can figure out how many frame (displays) have been skipped total; 9 times out of every 10 minutes
			const int32 NumFramesSkippedTotal = NumTimesSkippedDroppingFrames * 9 * NumberOfTimecodesToDrop;

			int32 OffsetFrame = FMath::Abs(InFrameNumber.Value);
			int FrameInTrueFrames = OffsetFrame % NumTrueFramesPerTenMinutes;

			// If we end up with a value of 0 or 1 (or 2 or 3 for 59.94) then we're not skipping timecode numbers this time.
			if (FrameInTrueFrames < NumberOfTimecodesToDrop)
			{
				OffsetFrame += NumFramesSkippedTotal;
			}
			else
			{
				// Each minute we slip a little bit more out of sync by a small amount, we just wait until we've accumulated enough error
				// to skip a whole frame and can catch up.
				const uint32 NumTrueFramesPerMinute = (uint32)FMath::FloorToInt(60 * (float)InFrameRate.AsDecimal());

				// Figure out which minute we are (0-9) to see how many to skip
				int32 CurrentMinuteOfTen = (FrameInTrueFrames - NumberOfTimecodesToDrop) / NumTrueFramesPerMinute;
				int NumAddedFrames = NumFramesSkippedTotal + (NumberOfTimecodesToDrop * CurrentMinuteOfTen);
				OffsetFrame += NumAddedFrames;
			}

			// Convert to negative timecode at the end if the original was negative.
			OffsetFrame *= FMath::Sign(InFrameNumber.Value);

			// Now that we've fudged what frames it thinks we're converting, we can do a standard Frame -> Timecode conversion.
			int32 Hours = (int32)FMath::RoundToZero(OffsetFrame / (double)NumberOfFramesInHour);
			int32 Minutes = (int32)FMath::RoundToZero(OffsetFrame / (double)NumberOfFramesInMinute) % 60;
			int32 Seconds = (int32)FMath::RoundToZero(OffsetFrame / (double)NumberOfFramesInSecond) % 60;
			int32 Frames = OffsetFrame % NumberOfFramesInSecond;

			return FTimecode(Hours, Minutes, Seconds, Frames, true);
		}
		else
		{
			// If we're in non-drop-frame we just convert straight through without fudging the frame numbers to skip certain timecodes.
			int32 Hours = (int32)FMath::RoundToZero(InFrameNumber.Value / (double)NumberOfFramesInHour);
			int32 Minutes = (int32)FMath::RoundToZero(InFrameNumber.Value / (double)NumberOfFramesInMinute) % 60;
			int32 Seconds = (int32)FMath::RoundToZero(InFrameNumber.Value / (double)NumberOfFramesInSecond) % 60;
			int32 Frames = InFrameNumber.Value % NumberOfFramesInSecond;

			return FTimecode(Hours, Minutes, Seconds, Frames, false);
		}
	}

	/**
	 * Create a FTimecode from a specific frame number at the given frame rate.
	 *
	 * @param InFrameNumber - The frame number to convert into a timecode. This should already be converted to InFrameRate's resolution.
	 * @param InFrameRate	- The framerate that this timecode is based in. This should be the playback framerate as it is used to determine
	 *						  when the Frame value wraps over.
	 */
	static FTimecode FromFrameNumber(const FFrameNumber& InFrameNumber, const FFrameRate& InFrameRate)
	{
		return FromFrameNumber(InFrameNumber, InFrameRate, UseDropFormatTimecode(InFrameRate));
	}

	/**
	 * Create a FTimecode from a specific frame time at the given frame rate.
	 *
	 * @param InFrameTime - The frame time to convert into a timecode. This should already be converted to InFrameTimes's resolution.
	 * @param InFrameRate - The framerate that this timecode is based in. This should be the playback framerate as it is used to determine
	 *					    when the Frame value wraps over.
	 * @param bDropFrame  - Optional parameter to indicate if we are in drop frame format.
	 */
	static FTimecode FromFrameTime(const FFrameTime& InFrameTime, const FFrameRate& InFrameRate, TOptional<bool> bDropFrame = {})
	{
		bool bShouldUseDropFrame = bDropFrame ? *bDropFrame : UseDropFormatTimecode(InFrameRate);
		FTimecode Timecode = FromFrameNumber(InFrameTime.FloorToFrame(), InFrameRate, bShouldUseDropFrame);
		Timecode.Subframe = InFrameTime.GetSubFrame();
		return Timecode;
	}

	/**
	 * Converts this Timecode back into a timespan at the given framerate, taking into account if this is a drop-frame format timecode.
	 */
	FTimespan ToTimespan(const FFrameRate& InFrameRate) const
	{
		const FFrameNumber ConvertedFrameNumber = ToFrameNumber(InFrameRate);
		const double NumberOfSeconds = ConvertedFrameNumber.Value * InFrameRate.AsInterval();
		return FTimespan::FromSeconds(NumberOfSeconds);
	}

	/**
	 * Create a FTimecode from a timespan at the given frame rate. Optionally supports creating a drop frame timecode,
	 * which drops certain timecode display numbers to help account for NTSC frame rates which are fractional.
	 *
	 * @param InTimespan	- The timespan to convert into a timecode.
	 * @param InFrameRate	- The framerate that this timecode is based in. This should be the playback framerate as it is used to determine
	 *						  when the Frame value wraps over.
	 * @param InbDropFrame	- If true, the returned timecode will drop the first two frames on every minute (except when Minute % 10 == 0)
	 *						  This is only valid for NTSC framerates (29.97, 59.94) and will assert if you try to create a drop-frame format
	 *						  from a non-valid framerate. All framerates can be represented when in non-drop frame format.
	 * @param InbRollover	- If true, the hours will be the modulo of 24.
	 */
	static FTimecode FromTimespan(const FTimespan& InTimespan, const FFrameRate& InFrameRate, bool InbDropFrame, bool InbRollover)
	{
		return FTimecode(InTimespan.GetTotalSeconds(), InFrameRate, InbDropFrame, InbRollover);
	}

	/**
	 * Create a FTimecode from a timespan at the given frame rate.
	 *
	 * @param InTimespan	- The timespan to convert into a timecode.
	 * @param InFrameRate	- The framerate that this timecode is based in. This should be the playback framerate as it is used to determine
	 *						  when the Frame value wraps over.
	 * @param InbRollover	- If true, the hours will be the modulo of 24.
	 */
	static FTimecode FromTimespan(const FTimespan& InTimespan, const FFrameRate& InFrameRate, bool InbRollover)
	{
		return FTimecode(InTimespan.GetTotalSeconds(), InFrameRate, UseDropFormatTimecode(InFrameRate), InbRollover);
	}

	/** Drop frame is only support for frame rate of 29.97 or 59.94. */
	static bool IsDropFormatTimecodeSupported(const double InRate)
	{
		return FMath::IsNearlyEqual(InRate, 30.0/1.001)
			|| FMath::IsNearlyEqual(InRate, 60.0/1.001);
	}

	/** Drop frame is only support for frame rate of 29.97 or 59.94. */
	static bool IsDropFormatTimecodeSupported(const FFrameRate& InFrameRate)
	{
		const double InRate = InFrameRate.AsDecimal();

		return IsDropFormatTimecodeSupported(InRate);
	}

	/** If the frame rate support drop frame format and the app wish to use drop frame format by default. */
	static bool UseDropFormatTimecode(const FFrameRate& InFrameRate)
	{
		return IsDropFormatTimecodeSupported(InFrameRate) && UseDropFormatTimecodeByDefaultWhenSupported();
	}

	/** By default, should we generate a timecode in drop frame format when the frame rate does support it. */
	static CORE_API bool UseDropFormatTimecodeByDefaultWhenSupported();

	/**
	 * Returns true if the given frame rate string represents a supported drop frame timecode frame rate, or false otherwise.
	 * 
	 * Drop frame timecode is only supported for NTSC_30 (29.97 FPS) or NTSC_60 (59.94 FPS) frame rates.
	 */
	static bool IsValidDropFormatTimecodeRate(const FString& InRateString)
	{
		const FString NTSC_30_DF = "29.97df";
		const FString NTSC_60_DF = "59.94df";

		return (InRateString == NTSC_30_DF || InRateString == NTSC_60_DF);
	}

	/**
	 * Get the Qualified Timecode formatted in HH:MM:SS:FF or HH:MM:SS;FF depending on if this represents drop-frame timecode or not.
	 * @param bForceSignDisplay - Forces the timecode to be prepended with a positive or negative sign.
								  Standard behavior is to only show the sign when the value is negative.
	 * @param bDisplaySubframe - If subframe decimal digits are desired.
	 */
	CORE_API FString ToString(bool bForceSignDisplay = false, bool bDisplaySubframe = false) const;

	/**
	 * Parses a string representation of a timecode.
	 *
	 * Supported SMPTE drop frame (DF) and non-drop frame (NDF) formats variations:
	 * - NDF: HH:MM:SS:FF
	 * - DF: HH:MM:SS;FF or HH:MM:SS.FF or HH;MM;SS;FF or HH.MM.SS.FF.
	 *
	 * The 2 digits per number is not enforced. It is possible to parse high frame numbers (above 60), such as for audio timecodes.
	 * Full SMPTE compliance is not ensured by the parser (i.e. greater than 24h, negative time and any number of frames per second).
	 *
	 * Sub-frame variation:
	 * Supports the sub-frame variation where the frame number is a decimal number: HH:MM:SS:FF.ZZ.
	 *
	 * Side effect of supporting sub-frame is that this function can't unambiguously parse partial timecodes.
	 * 
	 * @param InTimecodeString A string representation of a timecode
	 * 
	 * @return Parsed timecode if valid.
	 */
	static CORE_API TOptional<FTimecode> ParseTimecode(const FStringView InTimecodeString);

	/**
	 * Will return true if the timecode represents a valid timecode value where
	 *
	 * Minutes is +/- [0, 59]
	 * Seconds is +/- [0, 59]
	 * Frames [INT_MIN, INT_MAX]
	 * Subframes >= 0
	 *
	 */
	bool IsValid() const
	{
		auto InAbsRange = [](int32 Val, int32 MinVal, int32 MaxVal)
		{
		    return FMath::Abs(Val) >= MinVal && FMath::Abs(Val) <= MaxVal;
		};
		return InAbsRange(Minutes, 0, 59) && InAbsRange(Seconds, 0, 59) && Subframe >= 0;
	}
public:

	/** IMPORTANT: If you change the struct data, ensure that you also update the version in NoExportTypes.h  */

	/** How many hours does this timecode represent */
	int32 Hours;

	/** How many minutes does this timecode represent */
	int32 Minutes;

	/** How many seconds does this timecode represent */
	int32 Seconds;

	/** How many frames does this timecode represent */
	int32 Frames;

	/** The subframe value provided when converting to FFrameTime */
	float Subframe = 0;

	/** If true, this Timecode represents a Drop Frame timecode used to account for fractional frame rates in NTSC play rates. */
	bool bDropFrameFormat;

	friend inline bool operator==(const FTimecode& A, const FTimecode& B)
	{
		return A.Hours == B.Hours && A.Minutes == B.Minutes && A.Seconds == B.Seconds && A.Frames == B.Frames && A.Subframe == B.Subframe;
	}

	friend inline bool operator!=(const FTimecode& A, const FTimecode& B)
	{
		return !(A == B);
	}

};
