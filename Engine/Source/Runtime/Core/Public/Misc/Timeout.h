// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Timespan.h"
#include "HAL/PlatformTime.h"
#include "Math/UnrealMathUtility.h"
#include "CoreTypes.h"

namespace UE
{
	/** 
	 * Utility class to create a timeout that will expire at a point in the future.
	 * Example usage:
	 *
	 *	FTimeout TimeoutFromTimespan(FTimespan::FromMilliseconds(2));
	 *	FTimeout TimeoutFromSeconds(0.002);
	 *	while (!TimeoutFromSeconds.IsExpired()) { ... }
	 */
	class FTimeout
	{
	public:

		/** Return true if elapsed time is greater than the initially requested timeout */
		bool IsExpired() const
		{
			// First two cases can skip the slow current time check
			if (WillNeverExpire())
			{
				return false;
			}
			else if (IsAlwaysExpired())
			{
				return true;
			}
			else
			{
				return FPlatformTime::Cycles64() > (StartCycles + TimeoutCycles);
			}
		}

		/** Create a timeout that will never return true for IsExpired */
		static FTimeout Never()
		{
			return FTimeout(FPlatformTime::Cycles64(), NeverExpireCycles);
		}

		/** Returns true if this was created from Never and does not need to be repeatedly checked */
		UE_FORCEINLINE_HINT bool WillNeverExpire() const
		{
			return TimeoutCycles == NeverExpireCycles;
		}

		/** Create a timeout that will always return true for IsExpired */
		static FTimeout AlwaysExpired()
		{
			return FTimeout(FPlatformTime::Cycles64(), 0);
		}

		/** Returns true if this was created from AlwaysExpired and does not need to be repeatedly checked */
		UE_FORCEINLINE_HINT bool IsAlwaysExpired() const
		{
			return TimeoutCycles == 0;
		}

		/** Set this timeout to explicitly expired without recalculating start time */
		void SetToExpired()
		{
			TimeoutCycles = 0;
		}


		// Preferred API for creating and querying using double seconds

		/** Construct a timeout that starts right now and will end after the passed in time in seconds */
		explicit FTimeout(double TimeoutSeconds)
			: StartCycles(FPlatformTime::Cycles64())
		{
			SetTimeoutSeconds(TimeoutSeconds);
		}

		/** Construct a timeout that started at the same time as BaseTimeout, but with a new duration */
		explicit FTimeout(const FTimeout& BaseTimeout, double TimeoutSeconds)
			: StartCycles(BaseTimeout.StartCycles)
		{
			SetTimeoutSeconds(TimeoutSeconds);
		}

		/** Returns time since the timeout was created, in seconds */
		double GetElapsedSeconds() const
		{
			// StartCycles can never be greater than current time as there is no way to construct a timeout starting in the future
			return FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartCycles);
		}

		/** Returns time left until the timeout expires (which can be negative) in seconds */
		double GetRemainingSeconds() const
		{
			if (WillNeverExpire())
			{
				return NeverExpireSeconds;
			}

			// We convert to double separately to avoid underflow on the cycles
			// This could also be done with some branches or treating cycles as signed int64
			return GetTimeoutSeconds() - GetElapsedSeconds();
		}

		/** Returns duration of timeout in seconds */
		double GetTimeoutSeconds() const
		{
			return FPlatformTime::ToSeconds64(TimeoutCycles);
		}

		/** Sets the timeout to new value in seconds */
		void SetTimeoutSeconds(double TimeoutSeconds)
		{
			if (TimeoutSeconds <= 0.0)
			{
				SetToExpired();
			}
			else
			{
				TimeoutCycles = FPlatformTime::SecondsToCycles64(TimeoutSeconds);
			}
		}

		/** Safely modify the remaining time by adding the delta time in seconds to the timeout */
		void ModifyTimeoutSeconds(double DeltaTimeoutSeconds)
		{
			if (IsAlwaysExpired() || WillNeverExpire())
			{
				return;
			}

			if (DeltaTimeoutSeconds >= 0.0)
			{
				TimeoutCycles += FPlatformTime::SecondsToCycles64(DeltaTimeoutSeconds);
			}
			else
			{
				uint64 RemovedCycles = FPlatformTime::SecondsToCycles64(-DeltaTimeoutSeconds);
				if (RemovedCycles >= TimeoutCycles)
				{
					SetToExpired();
				}
				else
				{
					TimeoutCycles -= RemovedCycles;
				}
			}
		}


		// Older API for creating and querying using FTimespan

		/** Construct a timeout that starts right now and will end after the passed in timespan */
		explicit FTimeout(FTimespan TimeoutValue)
			: StartCycles(FPlatformTime::Cycles64())
		{
			if (TimeoutValue == FTimespan::MaxValue())
			{
				TimeoutCycles = NeverExpireCycles;
			}
			else
			{
				SetTimeoutSeconds(TimeoutValue.GetTotalSeconds());
			}
		}

		/** Returns time since the timeout was created, as a timespan */
		FTimespan GetElapsedTime() const
		{
			return FTimespan::FromSeconds(GetElapsedSeconds());
		}

		/** Returns time left until the timeout expires (which can be negative) as a timespan */
		FTimespan GetRemainingTime() const
		{
			if (WillNeverExpire())
			{
				return FTimespan::MaxValue();
			}

			return FTimespan::FromSeconds(GetRemainingSeconds());
		}

		/** Returns duration of timeout as a timespan */
		FTimespan GetTimeoutValue() const
		{
			if (WillNeverExpire())
			{
				return FTimespan::MaxValue();
			}
			
			return FTimespan::FromSeconds(GetTimeoutSeconds());
		}

		/**
		 * Intended for use in waiting functions, e.g. `FEvent::Wait()`
		 * returns the whole number (rounded up) of remaining milliseconds, clamped into [0, MAX_uint32] range
		 */
		uint32 GetRemainingRoundedUpMilliseconds() const
		{
			if (WillNeverExpire())
			{
				return MAX_uint32;
			}

			int64 RemainingTicks = GetRemainingTime().GetTicks();
			int64 RemainingMsecs = FMath::DivideAndRoundUp(RemainingTicks, ETimespan::TicksPerMillisecond);
			int64 RemainingMsecsClamped = FMath::Clamp<int64>(RemainingMsecs, 0, MAX_uint32);
			return (uint32)RemainingMsecsClamped;
		}

		friend bool operator==(FTimeout Left, FTimeout Right)
		{
			// Timeout cycles need to match which handles differentiating between always and never
			// For normal timeouts, also check the start cycles
			return Left.TimeoutCycles == Right.TimeoutCycles 
				&& (Left.WillNeverExpire() || Left.IsAlwaysExpired() || Left.StartCycles == Right.StartCycles);
		}

		friend bool operator!=(FTimeout Left, FTimeout Right)
		{
			return !operator==(Left, Right);
		}

	private:
		FTimeout(uint64 StartValue, uint64 TimeoutValue)
			: StartCycles(StartValue)
			, TimeoutCycles(TimeoutValue)
		{
		}

		static constexpr uint64 NeverExpireCycles = TNumericLimits<uint64>::Max();
		static constexpr double NeverExpireSeconds = TNumericLimits<double>::Max();

		// Value of FPlatformTime::Cycles64 at timeout creation, cannot be directly converted to seconds
		uint64 StartCycles = 0;

		// Length of timeout, can be converted to seconds as it is relative to StartCycles
		uint64 TimeoutCycles = 0;
	};
}
