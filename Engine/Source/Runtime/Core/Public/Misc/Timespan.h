// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Interval.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"

class FArchive;
class FOutputDevice;
class FString;
class UObject;
template <typename ElementType> struct TIntervalTraits;


/**
 * Time span related constants.
 */
namespace ETimespan
{
	/** The maximum number of ticks that can be represented in FTimespan. */
	inline constexpr int64 MaxTicks = 9223372036854775807;

	/** The minimum number of ticks that can be represented in FTimespan. */
	inline constexpr int64 MinTicks = -9223372036854775807 - 1;

	/** The number of nanoseconds per tick. */
	inline constexpr int64 NanosecondsPerTick = 100;

	/** The number of timespan ticks per day. */
	inline constexpr int64 TicksPerDay = 864000000000;

	/** The number of timespan ticks per hour. */
	inline constexpr int64 TicksPerHour = 36000000000;

	/** The number of timespan ticks per microsecond. */
	inline constexpr int64 TicksPerMicrosecond = 10;

	/** The number of timespan ticks per millisecond. */
	inline constexpr int64 TicksPerMillisecond = 10000;

	/** The number of timespan ticks per minute. */
	inline constexpr int64 TicksPerMinute = 600000000;

	/** The number of timespan ticks per second. */
	inline constexpr int64 TicksPerSecond = 10000000;

	/** The number of timespan ticks per week. */
	inline constexpr int64 TicksPerWeek = 6048000000000;

	/** The number of timespan ticks per year (365 days, not accounting for leap years). */
	inline constexpr int64 TicksPerYear = 365 * TicksPerDay;
}


/**
 * Implements a time span.
 *
 * A time span is the difference between two dates and times. For example, the time span between
 * 12:00:00 January 1, 2000 and 18:00:00 January 2, 2000 is 30.0 hours. Time spans are measured in
 * positive or negative ticks depending on whether the difference is measured forward or backward.
 * Each tick has a resolution of 0.1 microseconds (= 100 nanoseconds).
 *
 * In conjunction with the companion class FDateTime, time spans can be used to perform date and time
 * based arithmetic, such as calculating the difference between two dates or adding a certain amount
 * of time to a given date.
 *
 * When initializing time span values from single components, consider using the FromHours, FromMinutes,
 * FromSeconds, Zero, MinValue and related methods instead of calling the overloaded constructors as
 * they will make your code easier to read and understand.
 *
 * @see FDateTime
 */
struct FTimespan
{
public:

	/** Default constructor (zero initialization). */
	FTimespan()
		: Ticks(0)
	{ }

	/**
	 * Create and initialize a new time interval with the specified number of ticks.
	 *
	 * For better readability, consider using MinValue, MaxValue and Zero.
	 *
	 * @param Ticks The number of ticks.
	 * @see MaxValue, MinValue, Zero
	 */
	FTimespan(int64 InTicks)
		: Ticks(InTicks)
	{
		check((InTicks >= ETimespan::MinTicks) && (InTicks <= ETimespan::MaxTicks));
	}

	/**
	 * Create and initialize a new time interval with the specified number of hours, minutes and seconds.
	 *
	 * For better readability, consider using FromHours, FromMinutes and FromSeconds.
	 *
	 * @param Hours The hours component.
	 * @param Minutes The minutes component.
	 * @param Seconds The seconds component.
	 * @see FromHours, FromMinutes, FromSeconds
	 */
	FTimespan(int32 Hours, int32 Minutes, int32 Seconds)
	{
		Assign(0, Hours, Minutes, Seconds, 0);
	}

	/**
	 * Create and initialize a new time interval with the specified number of days, hours, minutes and seconds.
	 *
	 * For better readability, consider using FromDays, FromHours, FromMinutes and FromSeconds.
	 *
	 * @param Days The days component.
	 * @param Hours The hours component.
	 * @param Minutes The minutes component.
	 * @param Seconds The seconds component.
	 * @see FromDays, FromHours, FromMinutes, FromSeconds
	 */
	FTimespan(int32 Days, int32 Hours, int32 Minutes, int32 Seconds)
	{
		Assign(Days, Hours, Minutes, Seconds, 0);
	}

	/**
	 * Create and initialize a new time interval with the specified number of days, hours, minutes and seconds.
	 *
	 * @param Days The days component.
	 * @param Hours The hours component.
	 * @param Minutes The minutes component.
	 * @param Seconds The seconds component.
	 * @param FractionNano The fractional seconds (in nanosecond resolution).
	 */
 	FTimespan(int32 Days, int32 Hours, int32 Minutes, int32 Seconds, int32 FractionNano)
 	{
 		Assign(Days, Hours, Minutes, Seconds, FractionNano);
 	}

public:

	/**
	 * Return the result of adding the given time span to this time span.
	 *
	 * @return A time span whose value is the sum of this time span and the given time span.
	 */
	FTimespan operator+(const FTimespan& Other) const
	{
		return FTimespan(Ticks + Other.Ticks);
	}

	/**
	 * Adds the given time span to this time span.
	 *
	 * @return This time span.
	 */
	FTimespan& operator+=(const FTimespan& Other)
	{
		Ticks += Other.Ticks;
		return *this;
	}

	/**
	 * Return the inverse of this time span.
	 *
	 * The value of this time span must be greater than FTimespan::MinValue(), or else an overflow will occur.
	 *
	 * @return Inverse of this time span.
	 */
	FTimespan operator-() const
	{
		return FTimespan(-Ticks);
	}

	/**
	 * Return the result of subtracting the given time span from this time span.
	 *
	 * @param Other The time span to compare with.
	 * @return A time span whose value is the difference of this time span and the given time span.
	 */
	FTimespan operator-(const FTimespan& Other) const
	{
		return FTimespan(Ticks - Other.Ticks);
	}

	/**
	 * Subtract the given time span from this time span.
	 *
	 * @param Other The time span to subtract.
	 * @return This time span.
	 */
	FTimespan& operator-=(const FTimespan& Other)
	{
		Ticks -= Other.Ticks;
		return *this;
	}

	/**
	 * Return the result of multiplying the this time span with the given scalar.
	 *
	 * @param Scalar The scalar to multiply with.
	 * @return A time span whose value is the product of this time span and the given scalar.
	 */
	FTimespan operator*(double Scalar) const
	{
		return FTimespan((int64)((double)Ticks * Scalar));
	}

	/**
	 * Multiply this time span with the given scalar.
	 *
	 * @param Scalar The scalar to multiply with.
	 * @return This time span.
	 */
	FTimespan& operator*=(double Scalar)
	{
		Ticks = (int64)((double)Ticks * Scalar);
		return *this;
	}

	/**
	 * Return the result of dividing the this time span by the given scalar.
	 *
	 * @param Scalar The scalar to divide by.
	 * @return A time span whose value is the quotient of this time span and the given scalar.
	 */
	FTimespan operator/(double Scalar) const
	{
		return FTimespan((int64)((double)Ticks / Scalar));
	}

	/**
	 * Divide this time span by the given scalar.
	 *
	 * @param Scalar The scalar to divide by.
	 * @return This time span.
	 */
	FTimespan& operator/=(double Scalar)
	{
		Ticks = (int64)((double)Ticks / Scalar);
		return *this;
	}

	/**
	 * Return the result of calculating the modulus of this time span with another time span.
	 *
	 * @param Other The time span to divide by.
	 * @return A time span representing the remainder of the modulus operation.
	 */
	FTimespan operator%(const FTimespan& Other) const
	{
		return FTimespan(Ticks % Other.Ticks);
	}

	/**
	 * Calculate this time span modulo another.
	 *
	 * @param Other The time span to divide by.
	 * @return This time span.
	 */
	FTimespan& operator%=(const FTimespan& Other)
	{
		Ticks = Ticks % Other.Ticks;
		return *this;
	}

	/**
	 * Compare this time span with the given time span for equality.
	 *
	 * @param Other The time span to compare with.
	 * @return true if the time spans are equal, false otherwise.
	 */
	bool operator==(const FTimespan& Other) const
	{
		return (Ticks == Other.Ticks);
	}

	/**
	 * Compare this time span with the given time span for inequality.
	 *
	 * @param Other The time span to compare with.
	 * @return true if the time spans are not equal, false otherwise.
	 */
	bool operator!=(const FTimespan& Other) const
	{
		return (Ticks != Other.Ticks);
	}

	/**
	 * Check whether this time span is greater than the given time span.
	 *
	 * @param Other The time span to compare with.
	 * @return true if this time span is greater, false otherwise.
	 */
	bool operator>(const FTimespan& Other) const
	{
		return (Ticks > Other.Ticks);
	}

	/**
	 * Check whether this time span is greater than or equal to the given time span.
	 *
	 * @param Other The time span to compare with.
	 * @return true if this time span is greater or equal, false otherwise.
	 */
	bool operator>=(const FTimespan& Other) const
	{
		return (Ticks >= Other.Ticks);
	}

	/**
	 * Check whether this time span is less than the given time span.
	 *
	 * @param Other The time span to compare with.
	 * @return true if this time span is less, false otherwise.
	 */
	bool operator<(const FTimespan& Other) const
	{
		return (Ticks < Other.Ticks);
	}

	/**
	 * Check whether this time span is less than or equal to the given time span.
	 *
	 * @param Other The time span to compare with.
	 * @return true if this time span is less or equal, false otherwise.
	 */
	bool operator<=(const FTimespan& Other) const
	{
		return (Ticks <= Other.Ticks);
	}

public:

	/**
	 * Export this time span value to a string.
	 *
	 * @param ValueStr Will hold the string value.
	 * @param DefaultValue The default value.
	 * @param Parent Not used.
	 * @param PortFlags Not used.
	 * @param ExportRootScope Not used.
	 * @return true on success, false otherwise.
	 * @see ImportTextItem
	 */
	CORE_API bool ExportTextItem(FString& ValueStr, FTimespan const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;

	/**
	 * Get the days component of this time span.
	 *
	 * @return Days component.
	 */
	int32 GetDays() const
	{
		return (int32)(Ticks / ETimespan::TicksPerDay);
	}

	/**
	 * Get a time span with the absolute value of this time span.
	 *
	 * This method may overflow the timespan if its value is equal to MinValue.
	 *
	 * @return Duration of this time span.
	 * @see MinValue
	 */
	FTimespan GetDuration()
	{
		return FTimespan(Ticks >= 0 ? Ticks : -Ticks);
	}

	/**
	 * Gets the fractional seconds (in microsecond resolution).
	 *
	 * @return Number of microseconds in fractional part.
	 * @see GetTotalMicroseconds
	 */
	int32 GetFractionMicro() const
	{
		return (int32)((Ticks % ETimespan::TicksPerSecond) / ETimespan::TicksPerMicrosecond);
	}

	/**
	 * Gets the fractional seconds (in millisecond resolution).
	 *
	 * @return Number of milliseconds in fractional part.
	 * @see GetTotalMilliseconds
	 */
	int32 GetFractionMilli() const
	{
		return (int32)((Ticks % ETimespan::TicksPerSecond) / ETimespan::TicksPerMillisecond);
	}

	/**
	 * Gets the fractional seconds (in nanosecond resolution).
	 *
	 * @return Number of nanoseconds in fractional part.
	 */
	int32 GetFractionNano() const
	{
		return (int32)((Ticks % ETimespan::TicksPerSecond) * ETimespan::NanosecondsPerTick);
	}

	/**
	 * Gets the fractional ticks (in 100 nanosecond resolution).
	 *
	 * @return Number of ticks in fractional part.
	 */
	int32 GetFractionTicks() const
	{
		return (int32)(Ticks % ETimespan::TicksPerSecond);
	}

	/**
	 * Gets the hours component of this time span.
	 *
	 * @return Hours component.
	 * @see GetTotalHours
	 */
	int32 GetHours() const
	{
		return (int32)((Ticks / ETimespan::TicksPerHour) % 24);
	}

	/**
	 * Get the minutes component of this time span.
	 *
	 * @return Minutes component.
	 * @see GetTotalMinutes
	 */
	int32 GetMinutes() const
	{
		return (int32)((Ticks / ETimespan::TicksPerMinute) % 60);
	}

	/**
	 * Get the seconds component of this time span.
	 *
	 * @return Seconds component.
	 * @see GetTotalSeconds
	 */
	int32 GetSeconds() const
	{
		return (int32)((Ticks / ETimespan::TicksPerSecond) % 60);
	}

	/**
	 * Get the number of ticks represented by this time span.
	 *
	 * @return Number of ticks.
	 */
	int64 GetTicks() const
	{
		return Ticks;
	}

	/**
	 * Get the total number of days represented by this time span.
	 *
	 * @return Number of days.
	 * @see GetDays
	 */
	double GetTotalDays() const
	{
		return ((double)Ticks / ETimespan::TicksPerDay);
	}

	/**
	 * Get the total number of hours represented by this time span.
	 *
	 * @return Number of hours.
	 * @see GetHours
	 */
	double GetTotalHours() const
	{
		return ((double)Ticks / ETimespan::TicksPerHour);
	}

	/**
	 * Get the total number of microseconds represented by this time span.
	 *
	 * @return Number of microseconds.
	 * @see GetFractionMicro
	 */
	double GetTotalMicroseconds() const
	{
		return ((double)Ticks / ETimespan::TicksPerMicrosecond);
	}

	/**
	 * Get the total number of milliseconds represented by this time span.
	 *
	 * @return Number of milliseconds.
	 * @see GetFractionMilli
	 */
	double GetTotalMilliseconds() const
	{
		return ((double)Ticks / ETimespan::TicksPerMillisecond);
	}

	/**
	 * Get the total number of minutes represented by this time span.
	 *
	 * @return Number of minutes.
	 * @see GetMinutes
	 */
	double GetTotalMinutes() const
	{
		return ((double)Ticks / ETimespan::TicksPerMinute);
	}

	/**
	 * Get the total number of seconds represented by this time span.
	 *
	 * @return Number of seconds.
	 * @see GetSeconds
	 */
	double GetTotalSeconds() const
	{
		return ((double)Ticks / ETimespan::TicksPerSecond);
	}

	/**
	 * Import a time span value from a text buffer.
	 *
	 * @param Buffer The text buffer to import from.
	 * @param PortFlags Not used.
	 * @param Parent Not used.
	 * @param ErrorText The output device for error logging.
	 * @return true on success, false otherwise.
	 * @see ExportTextItem
	 */
	CORE_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);

	/**
	 * Check whether this time span is zero.
	 *
	 * @return true if the time span is zero, false otherwise.
	 * @see Zero
	 */
	bool IsZero() const
	{
		return (Ticks == 0LL);
	}

	/**
	 * Serialize this time span from or into the specified archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @return true on success, false otherwise.
	 */
	CORE_API bool Serialize(FArchive& Ar);

	/**
	* Serializes this date and time for network transmission
	*/
	CORE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/**
	 * Return the string representation of this time span using a default format.
	 *
	 * The returned string has the following format:
	 *		p[d.]hh:mm:ss.fff
	 *
	 * Note that 'p' is the plus or minus sign, and the date component is
	 * omitted for time spans that are shorter than one day.
	 *
	 * Examples:
	 *      -42.15:11:36.457 (45 days, 15 hours, 11 minutes, 36.457 seconds in the past)
	 *      +42.15:11:36.457 (45 days, 15 hours, 11 minutes, 36.457 seconds in the future)
	 *      +15:11:36.457 (15 hours, 11 minutes, 36.457 seconds in the future)
	 *      +00:11:36.457 (11 minutes, 36.457 seconds in the future)
	 *      +00:00:36.457 (36.457 seconds in the future)
	 *
	 * @return String representation.
	 * @see Parse
	 */
	CORE_API FString ToString() const;

	/**
	 * Convert this time span to its string representation.
	 *
	 * The following formatting codes are available:
	 *		%d - prints the days component
	 *		%D - prints the zero-padded days component (00000000..10675199)
	 *		%h - prints the zero-padded hours component (00..23)
	 *		%m - prints the zero-padded minutes component (00..59)
	 *		%s - prints the zero-padded seconds component (00..59)
	 *		%f - prints the zero-padded fractional seconds (000..999)
	 *		%u - prints the zero-padded fractional seconds (000000..999999)
	 *		%n - prints the zero-padded fractional seconds (000000000..999999999)
	 *
	 * Depending on whether the time span is positive or negative, a plus or minus
	 * sign character will always be added in front of the generated string.
	 *
	 * @param Format The format of the returned string.
	 * @return String representation.
	 * @see Parse
	 */
	CORE_API FString ToString(const TCHAR* Format) const;

public:

	/**
	 * Create a time span that represents the specified number of days.
	 *
	 * @param Days The number of days.
	 * @return Time span.
	 * @see FromHours, FromMicroseconds, FromMilliseconds, FromMinutes, FromSeconds
	 */
	static FTimespan FromDays(double Days)
	{
		return FTimespan((int64)FMath::FloorToDouble(Days * ETimespan::TicksPerDay + 0.5));
	}

	/**
	 * Create a time span that represents the specified number of hours.
	 *
	 * @param Hours The number of hours.
	 * @return Time span.
	 * @see FromDays, FromMicroseconds, FromMilliseconds, FromMinutes, FromSeconds
	 */
	static FTimespan FromHours(double Hours)
	{
		return FTimespan((int64)FMath::FloorToDouble(Hours * ETimespan::TicksPerHour + 0.5));
	}

	/**
	 * Create a time span that represents the specified number of microseconds.
	 *
	 * @param Microseconds The number of microseconds.
	 * @return Time span.
	 * @see FromDays, FromHours, FromMinutes, FromSeconds, FromMilliseconds
	 */
	static FTimespan FromMicroseconds(double Microseconds)
	{
		return FTimespan((int64)FMath::FloorToDouble(Microseconds * ETimespan::TicksPerMicrosecond + 0.5));
	}

	/**
	 * Create a time span that represents the specified number of milliseconds.
	 *
	 * @param Milliseconds The number of milliseconds.
	 * @return Time span.
	 * @see FromDays, FromHours, FromMicroseconds, FromMinutes, FromSeconds
	 */
	static FTimespan FromMilliseconds(double Milliseconds)
	{
		return FTimespan((int64)FMath::FloorToDouble(Milliseconds * ETimespan::TicksPerMillisecond + 0.5));
	}

	/**
	 * Create a time span that represents the specified number of minutes.
	 *
	 * @param Minutes The number of minutes.
	 * @return Time span.
	 * @see FromDays, FromHours, FromMicroseconds, FromMilliseconds, FromSeconds
	 */
	static FTimespan FromMinutes(double Minutes)
	{
		return FTimespan((int64)FMath::FloorToDouble(Minutes * ETimespan::TicksPerMinute + 0.5));
	}

	/**
	 * Create a time span that represents the specified number of seconds.
	 *
	 * @param Seconds The number of seconds.
	 * @return Time span.
	 * @see FromDays, FromHours, FromMicroseconds, FromMilliseconds, FromMinutes
	 */
	static FTimespan FromSeconds(double Seconds)
	{
		return FTimespan((int64)FMath::FloorToDouble(Seconds * ETimespan::TicksPerSecond + 0.5));
	}

	/**
	 * Return the maximum time span value.
	 *
	 * The maximum time span value is slightly more than 10,675,199 days.
	 *
	 * @return Maximum time span.
	 * @see MinValue,Zero
	 */
	static FTimespan MaxValue()
	{
		return FTimespan(ETimespan::MaxTicks);
	}

	/**
	 * Return the minimum time span value.
	 *
	 * The minimum time span value is slightly less than -10,675,199 days.
	 *
	 * @return Minimum time span.
	 * @see MaxValue, ZeroValue
	 */
	static FTimespan MinValue()
	{
		return FTimespan(ETimespan::MinTicks);
	}

	/**
	 * Convert a string to a time span.
	 *
	 * The string must be in one of the following formats:
	 *    p[d.]hh::mm::ss.fff
	 *    p[d.]hh::mm::ss.uuuuuu
	 *    p[d.]hh::mm::ss.nnnnnnnnn
	 *
	 * Note that 'p' is the plus or minus sign, and the date component may be
	 * omitted for time spans that are shorter than one day.
	 *
	 * @param TimespanString The string to convert.
	 * @param OutTimespan Will contain the parsed time span.
	 * @return true if the string was converted successfully, false otherwise.
	 * @see ToString
	 */
	static CORE_API bool Parse(const FString& TimespanString, FTimespan& OutTimespan);

	/**
	 * Ratio between two time spans (handles zero values).
	 *
	 * @param Dividend The dividend.
	 * @param Divisor The divisor.
	 * @return The quotient, i.e. Dividend / Divisor.
	 */
	static double Ratio(FTimespan Dividend, FTimespan Divisor)
	{
		if (Divisor == FTimespan::Zero())
		{
			return 0.0;
		}

		return (double)Dividend.GetTicks() / (double)Divisor.GetTicks();
	}

	/**
	 * Return the zero time span value.
	 *
	 * The zero time span value can be used in comparison operations with other time spans.
	 *
	 * @return Zero time span.
	 * @see IsZero, MaxValue, MinValue
	 */
	static FTimespan Zero()
	{
		return FTimespan(0);
	}

public:

	friend class UObject;

	/**
	 * Serialize the given time span from or into the specified archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param Timespan The time span value to serialize.
	 * @return The archive.
	 */
	friend CORE_API FArchive& operator<<(FArchive& Ar, FTimespan& Timespan);

	/**
	 * Get the hash for the specified time span.
	 *
	 * @param Timespan The timespan to get the hash for.
	 * @return Hash value.
	 */
	friend CORE_API uint32 GetTypeHash(const FTimespan& Timespan);

protected:

	/**
	 * Assign the specified components to this time span.
	 *
	 * @param Days The days component.
	 * @param Hours The hours component.
	 * @param Minutes The minutes component.
	 * @param Seconds The seconds component.
	 * @param FractionNano The fractional seconds (in nanosecond resolution).
	 */
	void CORE_API Assign(int32 Days, int32 Hours, int32 Minutes, int32 Seconds, int32 FractionNano);

private:
	friend struct Z_Construct_UScriptStruct_FTimespan_Statics;

private:

	/** The time span value in 100 nanoseconds resolution. */
	int64 Ticks;
};

template <>
struct TIntervalTraits<FTimespan>
{
	static FTimespan Max()
	{
		return FTimespan::MaxValue();
	}

	static FTimespan Lowest()
	{
		return FTimespan::MinValue();
	}
};

/**
 * Pre-multiply a time span with the given scalar.
 *
 * @param Scalar The scalar to pre-multiply with.
 * @param Timespan The time span to multiply.
 */
inline FTimespan operator*(float Scalar, const FTimespan& Timespan)
{
	return Timespan.operator*(Scalar);
}
