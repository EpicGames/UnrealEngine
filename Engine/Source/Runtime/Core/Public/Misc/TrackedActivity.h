// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

/**
* Enum to specify status light for an activity
* Is used by the new console to show a colored dot in front of the tracked activity entry
*/
enum class ETrackedActivityLight : uint8
{
	None,
	Red,
	Yellow,
	Green,
	Inherit,
};

/**
 * Tracked Activity is used to be able to visualize on a semi-high level what is going on in the process.
 * It is very useful when wanting to show the status of online, or if the runtime is waiting on loading something
 * When new console is enabled tracked activities show at the bottom of the window under the log
 * Tracked Activities can be created/updated/destroyed on multiple threads
 */
class FTrackedActivity : public TSharedFromThis<FTrackedActivity>
{
public:
	/**
	* Enum to specify status type of activity
	* In the new console, 'Activity' will make sure tracked activity appears to the left of the window
	* 'Info' appears to the right and use less width. Typically use 'Info' for things that are more static
	*/
	enum class EType
	{
		Activity,
		Info,
		Debug,
	};

	using ELight = ETrackedActivityLight;

	/** Ctor
	* @param Name Name of tracked activity
	* @param Status Initial status of tracked activity
	* @param Light for the activity. Will show as a dot in front of activity in console
	* @param Type Decides where activity information show in console. Activity is to the left, Info to the right
	* @param SortValue Decides in what order within type the activity will show in console. Lower value means earlier
	*/
	CORE_API FTrackedActivity(const TCHAR* Name, const TCHAR* Status = TEXT(""), ELight Light = ELight::None, EType Type = EType::Activity, int32 SortValue = 100);

	/** Dtor */
	CORE_API ~FTrackedActivity();


	/**
	* Pushes new status on to tracked activity, will require a pop to get back to previous status
	* ShowParent can be used to make sure Status of parent is visible in front of status of pushed scope.
	*/
	CORE_API uint32	Push(const TCHAR* Status, bool bShowParent = false, ELight Light = ELight::Inherit);
	CORE_API void	Pop();

	/** Updates status. If Index is ~0u entry at top of stack will be updated (which is the one showing) */
	CORE_API void	Update(const TCHAR* Status, uint32 Index = ~0u);
	CORE_API void	Update(const TCHAR* Status, ELight Light, uint32 Index = ~0u);
	CORE_API void	Update(ELight Light, uint32 Index = ~0u);


	/**
	* Process Engine Activity. General status of the Engine. Used by Engine initialization etc.
	* By default, status light will stay yellow until main threads hits Tick update, then it turns green.
	*/
	static CORE_API FTrackedActivity& GetEngineActivity();

	/** I/O Activity.Shows current I / O operation.If plugin / game have their own I / O, scopes will need to be manually added */
	static CORE_API FTrackedActivity& GetIOActivity();



	struct FInfo
	{
		const TCHAR* Name;
		const TCHAR* Status;
		ELight Light; // Can not be "Inherit"
		EType Type;
		int32 SortValue;
		uint32 Id; // Unique Id for Activity (Is thread safe counter starting at 1)
	};

	/**
	* Traverses all FTrackedActivities in the order they were added.
	*/
	static CORE_API void TraverseActivities(const TFunction<void(const FInfo& Info)>& Func);



	/** Enum used for Event listener to identify type of activity change */
	enum class EEvent : int32
	{
		Added = 1,
		Removed = 2,
		Changed = 3
	};

	/** Register listener that can track adds, removes and changes when they happen.
	* Listener will be called from the same thread as the event happens, so make sure listener is threadsafe.
	*/
	static CORE_API void RegisterEventListener(TUniqueFunction<void(EEvent Event, const FInfo& Info)>&& Func, uint32 MaxDepth = ~0u);
	static CORE_API void UnregisterEventListener();


private:
	FTrackedActivity(const FTrackedActivity& O) = delete;
	FTrackedActivity& operator=(const FTrackedActivity& O) = delete;

	void* Internal;
};


/**
* RAII class that calls push in ctor and pop in dtor.
*/
class FTrackedActivityScope
{
public:
	CORE_API FTrackedActivityScope(FTrackedActivity& Activity, const TCHAR* Status, bool bShowParent = false, FTrackedActivity::ELight Light = FTrackedActivity::ELight::Inherit);
	CORE_API ~FTrackedActivityScope();
private:
	FTrackedActivity& Activity;
};


/**
* Enabled tracking of IO
*/
#ifndef UE_ENABLE_TRACKED_IO
	#if UE_BUILD_SHIPPING
		#define UE_ENABLE_TRACKED_IO 0
		#define UE_SCOPED_IO_ACTIVITY(...)
	#else
		#define UE_ENABLE_TRACKED_IO 1
		#define UE_SCOPED_IO_ACTIVITY(...) FTrackedActivityScope ANONYMOUS_VARIABLE(IOActivity_)(FTrackedActivity::GetIOActivity(), __VA_ARGS__);
	#endif
#endif
