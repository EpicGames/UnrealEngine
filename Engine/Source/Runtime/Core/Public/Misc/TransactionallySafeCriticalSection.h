// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM.h"

#if UE_AUTORTFM
#include "Async/TransactionallySafeRecursiveMutex.h"
#else
#include "HAL/CriticalSection.h"
#endif // UE_AUTORTFM

#if UE_AUTORTFM
using FTransactionallySafeCriticalSection = ::UE::FTransactionallySafeRecursiveMutex;
#else
using FTransactionallySafeCriticalSection = ::FCriticalSection;
#endif // UE_AUTORTFM

