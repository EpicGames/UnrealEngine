// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM.h"
#include "HAL/Platform.h"

#if UE_AUTORTFM
#include "Async/TransactionallySafeSharedMutex.h"
#else
#include "HAL/CriticalSection.h"
#endif // UE_AUTORTFM


#if UE_AUTORTFM

// Transactionally-safe RWLocks are implemented in terms of FTransactionallySafeSharedMutex.
// Unfortunately, the method names differ slightly between RWLock and SharedMutex, so we adapt the names here.
class FTransactionallySafeRWLock : ::UE::FTransactionallySafeSharedMutex
{
private:
	using Super = ::UE::FTransactionallySafeSharedMutex;

public:
	UE_REWRITE void ReadLock()
	{
		Super::LockShared();
	}

	UE_REWRITE void ReadUnlock()
	{
		Super::UnlockShared();
	}

	UE_REWRITE void WriteLock()
	{
		Super::Lock();
	}

	UE_REWRITE void WriteUnlock()
	{
		Super::Unlock();
	}

	UE_REWRITE bool TryReadLock()
	{
		return Super::TryLockShared();
	}

	UE_REWRITE bool TryWriteLock()
	{
		return Super::TryLock();
	}
};

#else
using FTransactionallySafeRWLock = ::FRWLock;
#endif