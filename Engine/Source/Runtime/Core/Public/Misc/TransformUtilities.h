// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"

namespace TransformUtilities
{
	CORE_API uint32 GetRoundedTransformCRC32(const FTransform& InTransform);
}