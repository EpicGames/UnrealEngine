// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTypeTraits.h"
#include "Containers/UnrealString.h"
#include "Templates/Function.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "HAL/TlsAutoCleanup.h"
#include "Misc/ScopeLock.h"

/**
 * Enumerates the scopes for instance creation in type containers.
 */
enum class ETypeContainerScope
{
	/** Create a new instance each time. */
	Instance,

	/** One singleton for the entire process. */
	Process,

	/** One singleton per thread. */
	Thread,
};


/**
 * Template for type containers.
 *
 * Type containers allow for configuring objects and their dependencies without actually
 * creating these objects. They fulfill a role similar to class factories, but without
 * being locked to a particular type of class. When passed into class constructors or
 * methods, type containers can facilitate the Inversion of Control (IoC) pattern.
 *
 * Since UE neither uses run-time type information nor pre-processes plain old C++ classes,
 * type names need to be exposed using the Expose_TNameOf macro in order to make them
 * registrable with type containers, i.e. Expose_TNameOf(FMyClass).
 *
 * Once a type is registered with a container, instances of that type can be retrieved from it.
 * There are currently three life time scopes available for instance creation:
 *
 *   1. Unique instance per process (aka. singleton),
 *      using RegisterClass(ETypeContainerScope::Process) or RegisterInstance()
 *   2. Unique instance per thread (aka. thread singleton),
 *      using RegisterClass(ETypeContainerScope::Thread)
 *   3. Unique instance per call (aka. class factory),
 *      using RegisterClass(ETypeContainerScope::Instance) or RegisterFactory()
 *
 * See the file TypeContainerTest.cpp for detailed examples on how to use each of these
 * type registration methods.
 *
 * In the interest of fast performance, the object pointers returned by type containers are
 * not thread-safe by default. If you intend to use the same type container and share its
 * objects from different threads, use TTypeContainer<ESPMode::ThreadSafe> instead, which
 * will then manage and return thread-safe versions of all object pointers.
 *
 * Note: Type containers depend on variadic templates and are therefore not available
 * on XboxOne at this time, which means they should only be used for desktop applications.
 */
template<ESPMode Mode = ESPMode::ThreadSafe>
class TTypeContainer
{
	/** Interface for object instance providers. */
	class IInstanceProvider
	{
	public:

		/** Virtual destructor. */
		virtual ~IInstanceProvider() = default;

		/**
		 * Gets an instance of a class.
		 *
		 * @return Shared pointer to the instance (must be cast to TSharedPtr<R, Mode>).
		 */
		virtual TSharedPtr<void, Mode> GetInstance() = 0;
	};

	/** Implements an instance provider that forwards instance requests to a factory function. */
	template<class T>
	struct TFunctionInstanceProvider
		: public IInstanceProvider
	{
		TFunctionInstanceProvider(TFunction<TSharedPtr<void, Mode>()>&& InCreateFunc)
			: CreateFunc(MoveTemp(InCreateFunc))
		{ }

		virtual ~TFunctionInstanceProvider() override = default;

		virtual TSharedPtr<void, Mode> GetInstance() override
		{
			return CreateFunc();
		}

		TFunction<TSharedPtr<void, Mode>()> CreateFunc;
	};


	/** Implements an instance provider that returns the same instance for all threads. */
	template<class T>
	struct TSharedInstanceProvider
		: public IInstanceProvider
	{
		TSharedInstanceProvider(const TSharedRef<T, Mode>& InInstance)
			: Instance(InInstance)
		{ }

		virtual ~TSharedInstanceProvider() = default;

		virtual TSharedPtr<void, Mode> GetInstance() override
		{
			return Instance;
		}

		TSharedRef<T, Mode> Instance;
	};


	/** Base class for per-thread instances providers. */
	struct FThreadInstanceProvider
		: public IInstanceProvider
	{
		FThreadInstanceProvider(TFunction<TSharedPtr<void, Mode>()>&& InCreateFunc)
			: CreateFunc(MoveTemp(InCreateFunc))
			, TlsSlot(FPlatformTLS::AllocTlsSlot())
		{ }

		virtual ~FThreadInstanceProvider() override
		{
			FPlatformTLS::FreeTlsSlot(TlsSlot);
		}

		TFunction<TSharedPtr<void, Mode>()> CreateFunc;
		uint32 TlsSlot;
	};


	/** Implements an instance provider that returns the same instance per thread. */
	template<class T>
	struct TThreadInstanceProvider
		: public FThreadInstanceProvider
	{
		typedef TTlsAutoCleanupValue<TSharedPtr<T, Mode>> TlsValueType;

		TThreadInstanceProvider(TFunction<TSharedPtr<void, Mode>()>&& InCreateFunc)
			: FThreadInstanceProvider(MoveTemp(InCreateFunc))
		{ }

		virtual ~TThreadInstanceProvider() override = default;

		virtual TSharedPtr<void, Mode> GetInstance() override
		{
			auto TlsValue = (TlsValueType*)FPlatformTLS::GetTlsValue(this->TlsSlot);

			if (TlsValue == nullptr)
			{
				TlsValue = new TlsValueType(StaticCastSharedPtr<T>(this->CreateFunc()));
				TlsValue->Register();
				FPlatformTLS::SetTlsValue(this->TlsSlot, TlsValue);
			}

			return TlsValue->Get();
		}
	};

public:

	/**
	 * Gets a shared pointer to an instance of the specified class.
	 *
	 * @param R The type of class to get an instance for.
	 * @param A shared reference to the instance.
	 * @see RegisterClass, RegisterDelegate, RegisterFactory, RegisterInstance
	 */
	template<class R>
	TSharedRef<R, Mode> GetInstance() const
	{
		FScopeLock Lock(&CriticalSection);
		{
			const TSharedPtr<IInstanceProvider>& Provider = Providers.FindRef(TNameOf<R>::GetName());
			check(Provider.IsValid());

			return StaticCastSharedPtr<R>(Provider->GetInstance()).ToSharedRef();
		}
	}

	/**
	 * Gets a shared reference to an instance of the specified class.
	 *
	 * Unlike GetInstance(), this function will assert if no instance was registered for
	 * the requested type of class using either RegisterClass() or RegisterInstance().
	 *
	 * @param R The type of class that an instance is being requested for.
	 * @param A shared pointer to the instance.
	 * @see GetInstance, RegisterClass, RegisterInstance
	 */
	template<class R>
	TSharedRef<R, Mode> GetInstanceRef() const
	{
		return GetInstance<R>().ToSharedRef();
	}

	/**
	 * Check whether the specified class has been registered.
	 *
	 * @param R The type of registered class to check.
	 * @return true if the class was registered, false otherwise.
	 */
	template<class R>
	bool IsRegistered() const
	{
		FScopeLock Lock(&CriticalSection);
		{
			return Providers.Contains(TNameOf<R>::GetName());
		}
	}

	/**
	 * Registers a class for instances of the specified class.
	 *
	 * @param R The type of class to register an instance class for.
	 * @param T Tye type of class to register (must be the same as or derived from R).
	 * @param P The constructor parameters that the class requires.
	 * @param Scope The scope at which instances must be unique (default = Process).
	 * @see RegisterDelegate, RegisterInstance, Unregister
	 */
	template<class R, class T, typename... P>
	void RegisterClass(ETypeContainerScope Scope = ETypeContainerScope::Process)
	{
		static_assert(TPointerIsConvertibleFromTo<T, R>::Value, "T must implement R");

		TSharedPtr<IInstanceProvider> Provider;
		
		switch(Scope)
		{
		case ETypeContainerScope::Instance:
			Provider = MakeShareable(
				new TFunctionInstanceProvider<T>(
					[this]() -> TSharedPtr<void, Mode> {
						return MakeShareable(new T(GetInstance<P>()...));
					}
				)
			);
			break;

		case ETypeContainerScope::Thread:
			Provider = MakeShareable(
				new TThreadInstanceProvider<T>(
					[this]() -> TSharedPtr<void, Mode> {
						return MakeShareable(new T(GetInstance<P>()...));
					}
				)
			);
			break;

		default:
			Provider = MakeShareable(
				new TSharedInstanceProvider<T>(
					MakeShareable(new T(GetInstance<P>()...))
				)
			);
			break;
		}

		AddProvider(TNameOf<R>::GetName(), Provider);
	}

	/**
	 * Register a factory delegate for the specified class.
	 *
	 * @param R The type of class to register an instance class for.
	 * @param D Tye type of factory delegate to register.
	 * @param P The parameters that the delegate requires.
	 * @param Delegate The delegate to register.
	 */
	template<class R, class D, typename... P>
	void RegisterDelegate(D Delegate)
	{
		static_assert(std::is_same_v<TSharedRef<R, Mode>, typename D::RetValType>, "Delegate return type must be TSharedPtr<R>");

		TSharedPtr<IInstanceProvider> Provider = MakeShareable(
			new TFunctionInstanceProvider<R>(
				[this, Delegate]() -> TSharedPtr<void, Mode> {
					return Delegate.Execute(GetInstance<P>()...);
				}
			)
		);

		AddProvider(TNameOf<R>::GetName(), Provider);
	}

	/**
	 * Register a factory function for the specified class.
	 *
	 * @param R The type of class to register the instance for.
	 * @param P Additional parameters that the factory function requires.
	 * @param CreateFunc The factory function that will create instances.
	 * @see RegisterClass, RegisterInstance, Unregister
	 */
	template<class R>
	void RegisterFactory(const TFunction<TSharedRef<R, Mode>()>& CreateFunc)
	{
		TSharedPtr<IInstanceProvider> Provider = MakeShareable(
			new TFunctionInstanceProvider<R>(
				[=, this]() -> TSharedPtr<void, Mode> {
					return CreateFunc();
				}
			)
		);

		AddProvider(TNameOf<R>::GetName(), Provider);
	}

	/**
	 * Register a factory function for the specified class.
	 *
	 * @param R The type of class to register the instance for.
	 * @param P0 The first parameter that the factory function requires.
	 * @param P Additional parameters that the factory function requires.
	 * @param CreateFunc The factory function that will create instances.
	 * @see RegisterClass, RegisterInstance, Unregister
	 */
	template<class R, typename P0, typename... P>
	void RegisterFactory(const TFunction<TSharedRef<R, Mode>(TSharedRef<P0, Mode>, TSharedRef<P, Mode>...)>& CreateFunc)
	{
		TSharedPtr<IInstanceProvider> Provider = MakeShareable(
			new TFunctionInstanceProvider<R>(
				[=, this]() -> TSharedPtr<void, Mode> {
					return CreateFunc(GetInstance<P0>(), GetInstance<P>()...);
				}
			)
		);

		AddProvider(TNameOf<R>::GetName(), Provider);
	}

#if (defined(_MSC_VER) && _MSC_VER >= 1900) || defined(__clang__)
	/**
	 * Register a factory function for the specified class.
	 *
	 * This is a Clang/VS2015-specific overload for handling raw function pointers.
	 *
	 * @param R The type of class to register the instance for.
	 * @param P Additional parameters that the factory function requires.
	 * @param CreateFunc The factory function that will create instances.
	 * @see RegisterClass, RegisterInstance, Unregister
	 */
	template<class R, typename... P>
	void RegisterFactory(TSharedRef<R, Mode> (*CreateFunc)(TSharedRef<P, Mode>...))
	{
		TSharedPtr<IInstanceProvider> Provider = MakeShareable(
			new TFunctionInstanceProvider<R>(
				[this, CreateFunc]() -> TSharedPtr<void, Mode> {
					return CreateFunc(GetInstance<P>()...);
				}
			)
		);

		AddProvider(TNameOf<R>::GetName(), Provider);
	}
#endif

	/**
	 * Registers an existing instance for the specified class.
	 *
	 * @param R The type of class to register the instance for.
	 * @param T The type of the instance to register (must be the same as or derived from R).
	 * @param Instance The instance of the class to register.
	 * @see RegisterClass, RegisterDelegate, Unregister
	 */
	template<class R, class T>
	void RegisterInstance(const TSharedRef<T, Mode>& Instance)
	{
		static_assert(TPointerIsConvertibleFromTo<T, R>::Value, "T must implement R");

		AddProvider(TNameOf<R>::GetName(), MakeShareable(new TSharedInstanceProvider<R>(Instance)));
	}

	/**
	 * Unregisters the instance or class for a previously registered class.
	 *
	 * @param R The type of class to unregister.
	 * @see RegisterClass, RegisterDelegate, RegisterInstance
	 */
	template<class R>
	void Unregister()
	{
		FScopeLock Lock(&CriticalSection);
		{
			Providers.Remove(TNameOf<R>::GetName());
		}
	}

protected:

	/**
	 * Add an instance provider to the container.
	 *
	 * @param Name The name of the type to add the provider for.
	 * @param Provider The instance provider.
	 */
	void AddProvider(const TCHAR* Name, const TSharedPtr<IInstanceProvider>& Provider)
	{
		FScopeLock Lock(&CriticalSection);
		{
			Providers.Add(Name, Provider);
		}
	}

private:

	/** Critical section for synchronizing access. */
	mutable FCriticalSection CriticalSection;

	/** Maps class names to instance providers. */
	TMap<FString, TSharedPtr<IInstanceProvider>> Providers;
};


/** For backwards compatibility. */
class FTypeContainer : public TTypeContainer<ESPMode::ThreadSafe> { };
