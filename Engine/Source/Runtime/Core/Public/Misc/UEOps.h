// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/SameAs.h"
#include "HAL/Platform.h"

/**
 * This file defines global comparison operators for all types containing certain named member functions.
 * These ensure that the operators and their variants are defined correctly based on a single customization
 * point and should result in faster overload resolution.
 *
 * Benefits:
 * - They're automatically marked up with things like [[nodiscard]], constexpr and UE_REWRITE, so you don't have to.
 * - It synthesises <, <=, >, and >= from a single UEOpLessThan, which C++20 does not (it only synthesises those from operator<=>).
 * - It validates your definitions to ensure they are const and self-consistent.
 * - It avoids overload resolution problems with multiple systems providing mutually-compatible templated operators,
 *     e.g. A::operator==(const T&) and B::operator==(const U&), where T matches with B and U matches with A.
 * - The more UEOps is used, the shorter error messages we get ("could be this overload, or this overload, or this overload...").
 * - It avoids friendship issues.
 * - It's syntactically a lot cleaner.
 * - (Unproven) It should be a lot less work for the compiler to do overload resolution on.
 * 
 * Usage:
 *
 * bool FMyType::UEOpEquals(const FMyType&) const;
 * bool FMyType::UEOpEquals(const OtherType&) const;
 *   Should be defined to opt into operator== and operator!=.
 *
 * bool FMyType::UEOpLessThan(const FMyType&) const;
 * bool FMyType::UEOpLessThan(const OtherType&) const;
 * bool FMyType::UEOpGreaterThan(const OtherType&) const;
 *   Should be defined to opt into operator<, operator<=, operator> and operator>=.
 *
 * bool FMyType::UEOpGreaterThan(const FMyType&) const;
 *   Will never be called if FMyType::UEOpLessThan is defined, so is redundant.
 *
 * When a type in a namespace uses UEOps functions, comparisons can fail when calling the operators from an
 * unrelated namespace, where the global operators are hidden and argument-dependent lookup cannot find the
 * operators because they are not in the same namespace as the type.
 *
 * UE_OPS_NAMESPACE_VISIBLE makes the operators visible in that type's namespace, allowing ADL to work as
 * intended. Example:
 *
 * namespace N1
 * {
 *     struct FMyType
 *     {
 *         bool UEOpEquals(const FMyType& Other) const;
 *     };
 *
 *     UE_OPS_NAMESPACE_VISIBLE(FMyType)
 * }
 *
 * namespace N2
 * {
 *     struct FOtherType;
 *     bool operator==(const FOtherType& Lhs, const FOtherType& Rhs);
 *
 *     bool AreDifferent(const N1::FMyType& Lhs, const N1::FMyType& Rhs)
 *     {
 *         return Lhs != Rhs; // without UE_OPS_NAMESPACE_VISIBLE above, this will not compile.
 *     }
 * }
 */


/**
 * On MSVC, operator!= isn't synthesized from operator== in some contexts.  This was found in a case of FObjectHandle:
 *
 * FObjectHandle NullObjectHandle = ...;
 * FObjectHandle TestLateResolveUnsafeObjectHandle = ...;
 * ...
 * CHECK(NullObjectHandle != TestLateResolveUnsafeObjectHandle); // catch2\internal\catch_decomposer.hpp(219,9): error C2678: binary '!=': no operator found which takes a left-hand operand of type 'FObjectHandle' (or there is no acceptable conversion)
 *
 * Additional overloads are added to work around this issue - they'll just make overload resolution errors slightly worse.  When the following ticket is fixed, these overloads can be removed.
 *
 * https://developercommunity.visualstudio.com/t/Synthesised-operator-not-considered-in/10945932
 */
#if defined(_MSC_VER) && !defined(__clang__)
	#define UE_OPS_WORKAROUND_PRIVATE 1
#else
	#define UE_OPS_WORKAROUND_PRIVATE 0
#endif

namespace UE::Core::Private
{
	// This only exists to bind to NULL, which is not perfectly-forwardable but which is commonly used in comparison operators.
	// The type is unique so that it can't be confused with a type that will actually be used by real user code, and is used to
	// create a pointer-to-member type which is highly unlikely to be deduced.
	struct FIncomplete;

	template <typename LhsType, typename RhsType>
	concept CWithUEOpEquals = requires(LhsType& Lhs, RhsType& Rhs)
	{
		{ Lhs.UEOpEquals(Rhs) } -> UE::CSameAs<bool>;
	};

	template <typename LhsType, typename RhsType>
	concept CWithConstUEOpEquals = requires(const LhsType& Lhs, const RhsType& Rhs)
	{
		{ Lhs.UEOpEquals(Rhs) } -> UE::CSameAs<bool>;
	};

	template <typename LhsType, typename RhsType>
	concept CWithUEOpLessThan = requires(LhsType& Lhs, RhsType& Rhs)
	{
		{ Lhs.UEOpLessThan(Rhs) } -> UE::CSameAs<bool>;
	};

	template <typename LhsType, typename RhsType>
	concept CWithConstUEOpLessThan = requires(const LhsType& Lhs, const RhsType& Rhs)
	{
		{ Lhs.UEOpLessThan(Rhs) } -> UE::CSameAs<bool>;
	};

	template <typename LhsType, typename RhsType>
	concept CWithUEOpGreaterThan = requires(LhsType& Lhs, RhsType& Rhs)
	{
		{ Lhs.UEOpGreaterThan(Rhs) } -> UE::CSameAs<bool>;
	};

	template <typename LhsType, typename RhsType>
	concept CWithConstUEOpGreaterThan = requires(const LhsType& Lhs, const RhsType& Rhs)
	{
		{ Lhs.UEOpGreaterThan(Rhs) } -> UE::CSameAs<bool>;
	};

	template <typename LhsType, typename RhsType>
	concept CWithMismatchedUEOpLess = CWithUEOpLessThan<LhsType, RhsType> && !CWithUEOpLessThan<RhsType, LhsType> && !CWithUEOpGreaterThan<LhsType, RhsType>;

	template <typename LhsType, typename RhsType>
	concept CWithMismatchedUEOpGreater = CWithUEOpGreaterThan<LhsType, RhsType> && !CWithUEOpGreaterThan   <RhsType, LhsType> && !CWithUEOpLessThan<LhsType, RhsType>;

	template <typename LhsType, typename RhsType>
	concept CWithMismatchedConstUEOpEquals = CWithUEOpEquals<LhsType, RhsType> && !CWithConstUEOpEquals<LhsType, RhsType>;

	template <typename LhsType, typename RhsType>
	concept CWithMismatchedConstUEOpLess = CWithUEOpLessThan<LhsType, RhsType> && !CWithConstUEOpLessThan<LhsType, RhsType>;

	template <typename LhsType, typename RhsType>
	concept CWithMismatchedConstUEOpGreater = CWithUEOpGreaterThan<LhsType, RhsType> && !CWithConstUEOpGreaterThan<LhsType, RhsType>;

	template <typename LhsType, typename RhsType>
	constexpr bool ValidateUEOpsEquals()
	{
		if constexpr (CWithMismatchedConstUEOpEquals<LhsType, RhsType> || CWithMismatchedConstUEOpEquals<RhsType, LhsType>)
		{
			static_assert(sizeof(LhsType*) == 0, "UEOpEquals should be const and accept a const argument");
			return false;
		}
		else
		{
			return true;
		}
	}

	template <typename LhsType, typename RhsType>
	constexpr bool ValidateUEOpsLessAndGreater()
	{
		if constexpr (CWithMismatchedConstUEOpLess<LhsType, RhsType> || CWithMismatchedConstUEOpLess<RhsType, LhsType>)
		{
			static_assert(sizeof(LhsType*) == 0, "UEOpLessThan should be const and accept a const argument");
			return false;
		}
		else if constexpr (CWithMismatchedConstUEOpGreater<LhsType, RhsType> || CWithMismatchedConstUEOpGreater<RhsType, LhsType>)
		{
			static_assert(sizeof(LhsType*) == 0, "UEOpGreaterThan should be const and accept a const argument");
			return false;
		}
		else if constexpr (CWithMismatchedUEOpLess<LhsType, RhsType> || CWithMismatchedUEOpLess<RhsType, LhsType>)
		{
			static_assert(sizeof(LhsType*) == 0, "UEOpLessThan should be matched with UEOpGreaterThan of the same argument type");
			return false;
		}
		else if constexpr (CWithMismatchedUEOpGreater<LhsType, RhsType> || CWithMismatchedUEOpGreater<RhsType, LhsType>)
		{
			static_assert(sizeof(LhsType*) == 0, "UEOpGreaterThan should be matched with UEOpLessThan of the same argument type");
			return false;
		}
		else
		{
			return true;
		}
	}
}

template <typename LhsType, typename RhsType>
	requires UE::Core::Private::CWithUEOpEquals<LhsType, RhsType> || UE::Core::Private::CWithUEOpEquals<RhsType, LhsType>
[[nodiscard]] UE_REWRITE constexpr bool operator==(const LhsType& Lhs, const RhsType& Rhs)
{
	if constexpr (!UE::Core::Private::ValidateUEOpsEquals<LhsType, RhsType>())
	{
		return false;
	}
	else if constexpr (UE::Core::Private::CWithUEOpEquals<LhsType, RhsType>)
	{
		return Lhs.UEOpEquals(Rhs);
	}
	else
	{
		return Rhs.UEOpEquals(Lhs);
	}
}

// This overload exists to support A == NULL, which is common in comparison operators, but NULL is not perfectly-forwardable.
template <typename LhsType>
	requires UE::Core::Private::CWithUEOpEquals<LhsType, decltype(nullptr)>
[[nodiscard]] UE_REWRITE constexpr bool operator==(const LhsType& Lhs, int UE::Core::Private::FIncomplete::* Rhs)
{
	if constexpr (!UE::Core::Private::ValidateUEOpsEquals<LhsType, decltype(nullptr)>())
	{
		return false;
	}
	else
	{
		return Lhs.UEOpEquals(nullptr);
	}
}

#if UE_OPS_WORKAROUND_PRIVATE
	template <typename LhsType, typename RhsType>
		requires UE::Core::Private::CWithUEOpEquals<LhsType, RhsType> || UE::Core::Private::CWithUEOpEquals<RhsType, LhsType>
	[[nodiscard]] UE_REWRITE constexpr bool operator!=(const LhsType& Lhs, const RhsType& Rhs)
	{
		if constexpr (!UE::Core::Private::ValidateUEOpsEquals<LhsType, RhsType>())
		{
			return false;
		}
		else if constexpr (UE::Core::Private::CWithUEOpEquals<LhsType, RhsType>)
		{
			return !Lhs.UEOpEquals(Rhs);
		}
		else
		{
			return !Rhs.UEOpEquals(Lhs);
		}
	}

	template <typename LhsType>
		requires UE::Core::Private::CWithUEOpEquals<LhsType, decltype(nullptr)>
	[[nodiscard]] UE_REWRITE constexpr bool operator!=(const LhsType& Lhs, int UE::Core::Private::FIncomplete::* Rhs)
	{
		if constexpr (!UE::Core::Private::ValidateUEOpsEquals<LhsType, decltype(nullptr)>())
		{
			return false;
		}
		else
		{
			return !Lhs.UEOpEquals(nullptr);
		}
	}

	template <typename RhsType>
		requires UE::Core::Private::CWithUEOpEquals<RhsType, decltype(nullptr)>
	[[nodiscard]] UE_REWRITE constexpr bool operator!=(int UE::Core::Private::FIncomplete::* Lhs, const RhsType& Rhs)
	{
		if constexpr (!UE::Core::Private::ValidateUEOpsEquals<decltype(nullptr), RhsType>())
		{
			return false;
		}
		else
		{
			return !Rhs.UEOpEquals(nullptr);
		}
	}
#endif

template <typename LhsType, typename RhsType>
	requires UE::Core::Private::CWithUEOpLessThan<LhsType, RhsType> || UE::Core::Private::CWithUEOpLessThan<RhsType, LhsType> || UE::Core::Private::CWithUEOpGreaterThan<LhsType, RhsType> || UE::Core::Private::CWithUEOpGreaterThan<RhsType, LhsType>
[[nodiscard]] UE_REWRITE constexpr bool operator<(const LhsType& Lhs, const RhsType& Rhs)
{
	if constexpr (!UE::Core::Private::ValidateUEOpsLessAndGreater<LhsType, RhsType>())
	{
		return false;
	}
	else if constexpr (UE::Core::Private::CWithUEOpLessThan<LhsType, RhsType>)
	{
		return Lhs.UEOpLessThan(Rhs);
	}
	else // if constexpr (UE::Core::Private::CWithUEOpGreaterThan<RhsType, LhsType>)
	{
		return Rhs.UEOpGreaterThan(Lhs);
	}
}

template <typename LhsType, typename RhsType>
	requires UE::Core::Private::CWithUEOpLessThan<LhsType, RhsType> || UE::Core::Private::CWithUEOpLessThan<RhsType, LhsType> || UE::Core::Private::CWithUEOpGreaterThan<LhsType, RhsType> || UE::Core::Private::CWithUEOpGreaterThan<RhsType, LhsType>
[[nodiscard]] UE_REWRITE constexpr bool operator>(const LhsType& Lhs, const RhsType& Rhs)
{
	if constexpr (!UE::Core::Private::ValidateUEOpsLessAndGreater<LhsType, RhsType>())
	{
		return false;
	}
	else if constexpr (UE::Core::Private::CWithUEOpLessThan<RhsType, LhsType>)
	{
		return Rhs.UEOpLessThan(Lhs);
	}
	else // if constexpr (UE::Core::Private::CWithUEOpGreaterThan<LhsType, RhsType>)
	{
		return Lhs.UEOpGreaterThan(Rhs);
	}
}

template <typename LhsType, typename RhsType>
	requires UE::Core::Private::CWithUEOpLessThan<LhsType, RhsType> || UE::Core::Private::CWithUEOpLessThan<RhsType, LhsType> || UE::Core::Private::CWithUEOpGreaterThan<LhsType, RhsType> || UE::Core::Private::CWithUEOpGreaterThan<RhsType, LhsType>
[[nodiscard]] UE_REWRITE constexpr bool operator>=(const LhsType& Lhs, const RhsType& Rhs)
{
	if constexpr (!UE::Core::Private::ValidateUEOpsLessAndGreater<LhsType, RhsType>())
	{
		return false;
	}
	else if constexpr (UE::Core::Private::CWithUEOpLessThan<LhsType, RhsType>)
	{
		return !Lhs.UEOpLessThan(Rhs);
	}
	else // if constexpr (UE::Core::Private::CWithUEOpGreaterThan<RhsType, LhsType>)
	{
		return !Rhs.UEOpGreaterThan(Lhs);
	}
}

template <typename LhsType, typename RhsType>
	requires UE::Core::Private::CWithUEOpLessThan<LhsType, RhsType> || UE::Core::Private::CWithUEOpLessThan<RhsType, LhsType> || UE::Core::Private::CWithUEOpGreaterThan<LhsType, RhsType> || UE::Core::Private::CWithUEOpGreaterThan<RhsType, LhsType>
[[nodiscard]] UE_REWRITE constexpr bool operator<=(const LhsType& Lhs, const RhsType& Rhs)
{
	if constexpr (!UE::Core::Private::ValidateUEOpsLessAndGreater<LhsType, RhsType>())
	{
		return false;
	}
	else if constexpr (UE::Core::Private::CWithUEOpLessThan<RhsType, LhsType>)
	{
		return !Rhs.UEOpLessThan(Lhs);
	}
	else // if constexpr (UE::Core::Private::CWithUEOpGreaterThan<LhsType, RhsType>)
	{
		return !Lhs.UEOpGreaterThan(Rhs);
	}
}

#if UE_OPS_WORKAROUND_PRIVATE
	// Allows UEOps to be used on the given type when called from an unrelated namespace.
	#define UE_OPS_NAMESPACE_VISIBLE(Type) \
		static_assert(sizeof(Type) > 0); /* We don't use the Type parameter right now - just check that the name exists and the macro is placed after the full type definition as a basic usability check */ \
		using ::operator==; \
		using ::operator!=; \
		using ::operator<; \
		using ::operator>; \
		using ::operator<=; \
		using ::operator>=;
#else
	// Allows UEOps to be used on the given type when called from an unrelated namespace.
	#define UE_OPS_NAMESPACE_VISIBLE(Type) \
		static_assert(sizeof(Type) > 0); /* We don't use the Type parameter right now - just check that the name exists and the macro is placed after the full type definition as a basic usability check */ \
		using ::operator==; \
		using ::operator<; \
		using ::operator>; \
		using ::operator<=; \
		using ::operator>=;
#endif

#undef UE_OPS_WORKAROUND_PRIVATE

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Templates/Requires.h"
#endif
