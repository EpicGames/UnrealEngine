// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Containers/Map.h"

/**
 * Dictionary of all the non-foreign projects for an engine installation, found by parsing .uprojectdirs files for source directories.
 */
class FUProjectDictionary
{
public:
	/** Scans the engine root directory for all the known projects. */
	CORE_API FUProjectDictionary(const FString& InRootDir);
	
	/** Refreshes the list of known projects */
	CORE_API void Refresh();

	/** Determines whether a project is a foreign project or not. */
	CORE_API bool IsForeignProject(const FString& ProjectFileName) const;

	/** Gets the project filename for the given game. Empty if not found. */
	CORE_API FString GetRelativeProjectPathForGame(const TCHAR* GameName, const FString& BaseDir) const;

	/** Gets the project filename for the given game. Empty if not found. */
	CORE_API FString GetProjectPathForGame(const TCHAR* GameName) const;

	/** Gets a list of all the known projects. */
	CORE_API TArray<FString> GetProjectPaths() const;

	/** Gets the project dictionary for the active engine installation. */
	static CORE_API FUProjectDictionary& GetDefault();

private:
	/** The root directory for this engine installation */
	FString RootDir;

	/** List of project root directories */
	TArray<FString> ProjectRootDirs;

	/** Map of short game names to full project paths. */
	TMap<FString, FString> ShortProjectNameDictionary;
};
