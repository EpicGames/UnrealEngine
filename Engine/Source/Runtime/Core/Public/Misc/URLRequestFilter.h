// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "Containers/Map.h"

namespace UE::Core
{

/**
 * A filter that can used to restrict URL schemes and domains to known-safe values.
 * Stores a map of schemes to domains: if the map is empty, no filtering is performed and any URL is allowed.
 * Each key is a scheme (such as "https"). If a scheme is present, it will be allowed.
 * Each value is an array of absolute domains or partial higher-level domains that are allowed for their scheme.
 *
 * Absolute domains do not start with a period and will be matched exactly with domains from the input URL,
 * for example, "epicgames.com".
 *
 * Partial domains start with a period and will be matched with the ending of the input domain.
 * Use these to allow all subdomains of the higher-level domain. For example, ".epicgames.com"
 *
 * To allow both an absolute domain and all of its subdomains, provide two entries in the array,
 * one with the period prefix and one without. For example, "epicgames.com" and ".epicgames.com".
 */
class FURLRequestFilter
{
public:
	using FRequestMap = TMap<const FString, const TArray<FString>>;

	FURLRequestFilter() = default;

	/**
	 * Initializes the allowlist map based on ini configs. Example syntax:
	 * 
	 * [ConfigSectionRootName https]
	 * !AllowedDomains=ClearArray
	 * +AllowedDomains=epicgames.com
	 * 
	 * [ConfigSectionRootName http]
	 * !AllowedDomains=ClearArray
	 * +AllowedDomains=epicgames.com
	 */
	CORE_API FURLRequestFilter(const TCHAR* ConfigSectionRootName, const FString& ConfigFileName);

	/**
	 * Initializes the allowlist based on an explicit scheme/domain map
	 */
	CORE_API FURLRequestFilter(const FRequestMap& InAllowedRequests);
	CORE_API FURLRequestFilter(FRequestMap&& InAllowedRequests);

	/**
	 * Update the allow list by corresponding section in a config file
	 */
	CORE_API void UpdateConfig(const TCHAR* ConfigSectionRootName, const FString& ConfigFileName);

	/**
	 * Check if allow list is empty
	 */
	bool IsEmpty() const { return AllowedRequests.IsEmpty(); }

	/**
	 * Matches InURL against the configued list of allowed URL schemes and domains. Returns true if the URL is allowed.
	 */
	CORE_API bool IsRequestAllowed(FStringView InURL) const;

private:
	FRequestMap AllowedRequests;
};

}
