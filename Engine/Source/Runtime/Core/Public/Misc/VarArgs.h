// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/*-----------------------------------------------------------------------------
VarArgs helper macros.
-----------------------------------------------------------------------------*/

// phantom definitions to help VAX parse our VARARG_* macros (VAX build 1440)
#ifdef VISUAL_ASSIST_HACK
	#define VARARG_DECL( FuncRet, StaticFuncRet, Return, FuncName, Pure, FmtType, ExtraDecl, ExtraCall ) FuncRet FuncName( ExtraDecl FmtType Fmt, ... )
	#define VARARG_BODY( FuncRet, FuncName, FmtType, ExtraDecl ) FuncRet FuncName( ExtraDecl FmtType Fmt, ... )
#endif // VISUAL_ASSIST_HACK

#define GET_TYPED_VARARGS(CharType, msg, msgsize, len, lastarg, fmt) \
	{ \
		va_list ap; \
		va_start(ap, lastarg); \
		TCString<CharType>::GetVarArgs(msg, msgsize, fmt, ap); \
		va_end(ap); \
	}

#define GET_VARARGS(     msg, msgsize, len, lastarg, fmt) UE_DEPRECATED_MACRO(5.4, "GET_VARARGS(...) has been deprecated - please use GET_TYPED_VARARGS(TCHAR, ...) instead")         GET_TYPED_VARARGS(TCHAR,    msg, msgsize, len, lastarg, fmt)
#define GET_VARARGS_WIDE(msg, msgsize, len, lastarg, fmt) UE_DEPRECATED_MACRO(5.4, "GET_VARARGS_WIDE(...) has been deprecated - please use GET_TYPED_VARARGS(WIDECHAR, ...) instead") GET_TYPED_VARARGS(WIDECHAR, msg, msgsize, len, lastarg, fmt)
#define GET_VARARGS_ANSI(msg, msgsize, len, lastarg, fmt) UE_DEPRECATED_MACRO(5.4, "GET_VARARGS_ANSI(...) has been deprecated - please use GET_TYPED_VARARGS(ANSICHAR, ...) instead") GET_TYPED_VARARGS(ANSICHAR, msg, msgsize, len, lastarg, fmt)

#define GET_TYPED_VARARGS_RESULT(CharType, msg, msgsize, len, lastarg, fmt, result) \
	{ \
		va_list ap; \
		va_start(ap, lastarg); \
		result = TCString<CharType>::GetVarArgs(msg, msgsize, fmt, ap); \
		if (result >= msgsize) \
		{ \
			result = -1; \
		} \
		va_end(ap); \
	}

#define GET_VARARGS_RESULT(     msg, msgsize, len, lastarg, fmt, result) UE_DEPRECATED_MACRO(5.4, "GET_VARARGS(...) has been deprecated - please use GET_TYPED_VARARGS(TCHAR, ...) instead") GET_TYPED_VARARGS_RESULT(TCHAR,    msg, msgsize, len, lastarg, fmt, result)
#define GET_VARARGS_RESULT_WIDE(msg, msgsize, len, lastarg, fmt, result) UE_DEPRECATED_MACRO(5.4, "GET_VARARGS_RESULT_WIDE(...) has been deprecated - please use GET_TYPED_VARARGS(WIDECHAR, ...) instead") GET_TYPED_VARARGS_RESULT(WIDECHAR, msg, msgsize, len, lastarg, fmt, result)
#define GET_VARARGS_RESULT_ANSI(msg, msgsize, len, lastarg, fmt, result) UE_DEPRECATED_MACRO(5.4, "GET_VARARGS_RESULT_ANSI(...) has been deprecated - please use GET_TYPED_VARARGS(ANSICHAR, ...) instead") GET_TYPED_VARARGS_RESULT(ANSICHAR, msg, msgsize, len, lastarg, fmt, result)
#define GET_VARARGS_RESULT_UTF8(msg, msgsize, len, lastarg, fmt, result) UE_DEPRECATED_MACRO(5.4, "GET_VARARGS_RESULT_UTF8(...) has been deprecated - please use GET_TYPED_VARARGS(UTF8CHAR, ...) instead") GET_TYPED_VARARGS_RESULT(UTF8CHAR, msg, msgsize, len, lastarg, fmt, result)

/*-----------------------------------------------------------------------------
Ugly VarArgs type checking (debug builds only).
-----------------------------------------------------------------------------*/

#define VARARG_EXTRA(...) __VA_ARGS__,
#define VARARG_NONE
#define VARARG_PURE =0
