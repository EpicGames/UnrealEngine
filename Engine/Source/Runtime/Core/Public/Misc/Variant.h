// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Misc/Timespan.h"
#include "UObject/NameTypes.h"
#include "Math/Vector2D.h"
#include "Math/IntRect.h"
#include "Containers/EnumAsByte.h"
#include "Math/IntVector.h"
#include "Math/Vector.h"
#include "Misc/DateTime.h"
#include "Math/Box.h"
#include "Math/BoxSphereBounds.h"
#include "Math/Vector4.h"
#include "Math/Plane.h"
#include "Math/Rotator.h"
#include "Math/Matrix.h"
#include "Math/Quat.h"
#include "Math/TwoVectors.h"
#include "Math/Transform.h"
#include "Misc/Guid.h"
#include "Math/RandomStream.h"
#include "Misc/NetworkGuid.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

/**
 * Enumerates the built-in types that can be stored in instances of FVariant.
 */
enum class EVariantTypes : int32
{
	Empty = 0,
	Ansichar = 1,
	Bool = 2,
	Box = 3,
	BoxSphereBounds = 4,
	ByteArray = 5,
	Color = 6,
	DateTime = 7,
	Double = 8,
	Enum = 9,
	Float = 10,
	Guid = 11,
	Int8 = 12,
	Int16 = 13,
	Int32 = 14,
	Int64 = 15,
	IntRect = 16,
	LinearColor = 17,
	Matrix = 18,
	Name = 19,
	Plane = 20,
	Quat = 21,
	RandomStream = 22,
	Rotator = 23,
	String = 24,
	Widechar = 25,
	Timespan = 26,
	Transform = 27,
	TwoVectors = 28,
	UInt8 = 29,
	UInt16 = 30,
	UInt32 = 31,
	UInt64 = 32,
	Vector = 33,
	Vector2d = 34,
	Vector4 = 35,
	IntPoint = 36,
	IntVector = 37,
	NetworkGUID = 38,

	Custom = 0x40
};


/**
 * Stub for variant type traits.
 *
 * Actual type traits need to be declared through template specialization for custom
 * data types that are to be used in FVariant. Traits for the most commonly used built-in
 * types are declared below.
 *
 * Complex types, such as structures and classes can be serialized into a byte array
 * and then assigned to a variant. Note that you will be responsible for ensuring
 * correct byte ordering when serializing those types.
 *
 * @param T The type to be used in FVariant.
 */
template<typename T> struct TVariantTraits
{
	static constexpr EVariantTypes GetType()
	{
		static_assert(!sizeof(T), "Variant trait must be specialized for this type.");
		return EVariantTypes::Empty;
	}
};


/**
 * Implements an extensible union of multiple types.
 *
 * Variant types can be used to store a range of different built-in types, as well as user
 * defined types. The values are internally serialized into a byte array, which means that
 * only FArchive serializable types are supported at this time.
 */
class FVariant
{
public:

	/** Default constructor. */
	FVariant()
		: Type(EVariantTypes::Empty)
	{ }

	FVariant(const FVariant&) = default;
	FVariant& operator=(const FVariant&) = default;
	~FVariant() = default;

	FVariant(FVariant&& Other)
		: Type(Other.Type)
		, Value(MoveTemp(Other.Value))
	{
		Other.Type = EVariantTypes::Empty;
	}

	FVariant& operator=(FVariant&& Other)
	{
		if (&Other != this)
		{
			Type = Other.Type;
			Value = MoveTemp(Other.Value);
			Other.Type = EVariantTypes::Empty;
		}
		return *this;
	}

	/**
	 * Creates and initializes a new instance with the specified value.
	 *
	 * @param InValue The initial value.
	 */
	template<typename T>
	FVariant( T InValue )
	{
		FMemoryWriter writer(Value, true);
		writer << InValue;

		Type = TVariantTraits<T>::GetType();
	}

	/**
	 * Creates and initializes a new instance from a byte array.
	 *
	 * Array values are passed straight through as an optimization. Please note that, if you
	 * serialize any complex types into arrays and then store them in FVariant, you will be
	 * responsible for ensuring byte ordering if the FVariant gets sent over the network.
	 *
	 * @param InValue- The initial value.
	 */
	FVariant( TArray<uint8>&& InArray )
		: Type(EVariantTypes::ByteArray)
		, Value(MoveTemp(InArray))
	{ }
	FVariant( const TArray<uint8>& InArray )
		: Type(EVariantTypes::ByteArray)
		, Value(InArray)
	{ }

	/**
	 * Creates and initializes a new instance from a TCHAR string.
	 *
	 * @param InString The initial value.
	 */
	FVariant( const TCHAR* InString )
	{
		*this = FString(InString);
	}

public:

	/**
	 * Assignment operator.
	 *
	 * @param T The type of the value to assign.
	 * @param InValue The value to assign.
	 * @return This instance.
	 */
	template<typename T>
	FVariant& operator=( T InValue )
	{
		FMemoryWriter Writer(Value, true);
		Writer << InValue;

		Type = TVariantTraits<T>::GetType();

		return *this;
	}

	/**
	 * Assignment operator for byte arrays.
	 *
	 * Array values are passed straight through as an optimization. Please note that, if you
	 * serialize any complex types into arrays and then store them in FVariant, you will be
	 * responsible for ensuring byte ordering if the FVariant gets sent over the network.
	 *
	 * @param InArray The byte array to assign.
	 * @return This instance.
	 */
	FVariant& operator=( TArray<uint8>&& InArray )
	{
		Type = EVariantTypes::ByteArray;
		Value = MoveTemp(InArray);

		return *this;
	}
	FVariant& operator=( const TArray<uint8>& InArray )
	{
		Type = EVariantTypes::ByteArray;
		Value = InArray;

		return *this;
	}

	/**
	 * Assignment operator for TCHAR strings.
	 *
	 * @param InString The value to assign.
	 * @return This instance.
	 */
	FVariant& operator=( const TCHAR* InString )
	{
		*this = FString(InString);

		return *this;
	}


	/**
	 * Implicit conversion operator.
	 *
	 * @param T The type to convert the value to.
	 * @return The value converted to the specified type.
	 */
	template<typename T>
	operator T() const
	{
		return GetValue<T>();
	}

public:

	/**
	 * Comparison operator for equality.
	 *
	 * @param Other The variant to compare with.
	 * @return true if the values are equal, false otherwise.
	 */
	bool operator==( const FVariant& Other ) const
	{
		return ((Type == Other.Type) && (Value == Other.Value));
	}

	/**
	 * Comparison operator for inequality.
	 *
	 * @param Other The variant to compare with.
	 * @return true if the values are not equal, false otherwise.
	 */
	bool operator!=( const FVariant& Other ) const
	{
		return ((Type != Other.Type) || (Value != Other.Value));
	}

public:

	/**
	 * Empties the value.
	 *
	 * @see IsEmpty
	 */
	void Empty()
	{
		Type = EVariantTypes::Empty;

		Value.Empty();
	}

	/**
	 * Checks whether the value is empty.
	 *
	 * @return true if the value is empty, false otherwise.
	 *
	 * @see Empty
	 */
	bool IsEmpty() const
	{
		return (Type == EVariantTypes::Empty);
	}

	/**
	 * Gets the stored value as a byte array.
	 *
	 * This method returns the internal representation of any value as an
	 * array of raw bytes. To retrieve values of type TArray<uint8> use
	 * GetValue<TArray<uint8>>() instead.
	 *
	 * @return Byte array.
	 * @see GetValue
	 */
	const TArray<uint8>& GetBytes() const
	{
		return Value;
	}

	/**
	 * Gets the stored value's size (in bytes).
	 *
	 * @return Size of the value.
	 * @see GetType, GetValue
	 */
	int32 GetSize() const
	{
		return Value.Num();
	}

	/**
	 * Gets the stored value's type.
	 *
	 * @return Type of the value.
	 * @see GetSize, GetValue
	 */
	EVariantTypes GetType() const
	{
		return Type;
	}

	/**
	 * Gets the stored value.
	 *
	 * This template function does not provide any automatic conversion between
	 * convertible types. The exact type of the value to be extracted must be known.
	 *
	 * @return The value.
	 * @see GetSize, GetType
	 */
	template<typename T>
	T GetValue() const
	{
		check((Type == TVariantTraits<T>::GetType()) || ((TVariantTraits<T>::GetType() == EVariantTypes::UInt8) && (Type == EVariantTypes::Enum)));

		T Result;

		FMemoryReader Reader(Value, true);
		Reader << Result;

		return Result;
	}

public:

	/**
	 * Serializes the given variant type from or into the specified archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param Variant The value to serialize.
	 * @return The archive.
	 */
	friend FArchive& operator<<( FArchive& Ar, FVariant& Variant )
	{
		return Ar << Variant.Type << Variant.Value;
	}

private:

	/** Holds the type of the variant. */
	EVariantTypes Type;

	/** Holds the serialized value. */
	TArray<uint8> Value;
};


/**
 * Gets the stored value for byte arrays.
 *
 * Array values are passed straight through as an optimization. Please note that, if you serialize
 * any complex types into arrays and then store them in FVariant, you will be responsible for
 * ensuring byte ordering if the FVariant gets send over the network.
 *
 * To retrieve any value as an array of serialized bytes, use GetBytes() instead.
 *
 * @return The byte array.
 * @see GetBytes
 */
template<>
inline TArray<uint8> FVariant::GetValue<TArray<uint8> >() const
{
	check(Type == EVariantTypes::ByteArray);

	return Value;
}


/* Default FVariant traits for built-in types
 *****************************************************************************/

/** Implements variant type traits for the built-in ANSICHAR type. */
template<> struct TVariantTraits<ANSICHAR>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Ansichar; }
};


/** Implements variant type traits for the built-in bool type. */
template<> struct TVariantTraits<bool>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Bool; }
};


/** Implements variant type traits for the built-in FBox type. */
template<> struct TVariantTraits<FBox>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Box; }
};


/** Implements variant type traits for the built-in FBoxSphereBounds type. */
template<> struct TVariantTraits<FBoxSphereBounds>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::BoxSphereBounds; }
};


/** Implements variant type traits for byte arrays. */
template<> struct TVariantTraits<TArray<uint8> >
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::ByteArray; }
};


/** Implements variant type traits for the built-in FColor type. */
template<> struct TVariantTraits<FColor>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Color; }
};


/** Implements variant type traits for the built-in FDateTime type. */
template<> struct TVariantTraits<FDateTime>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::DateTime; }
};


/** Implements variant type traits for the built-in double type. */
template<> struct TVariantTraits<double>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Double; }
};


/** Implements variant type traits for enumeration types. */
template<typename EnumType> struct TVariantTraits<TEnumAsByte<EnumType> >
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Enum; }
};


/** Implements variant type traits for the built-in float type. */
template<> struct TVariantTraits<float>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Float; }
};


/** Implements variant type traits for the built-in FGuid type. */
template<> struct TVariantTraits<FGuid>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Guid; }
};


/** Implements variant type traits for the built-in int8 type. */
template<> struct TVariantTraits<int8>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Int8; }
};


/** Implements variant type traits for the built-in int16 type. */
template<> struct TVariantTraits<int16>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Int16; }
};


/** Implements variant type traits for the built-in int32 type. */
template<> struct TVariantTraits<int32>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Int32; }
};


/** Implements variant type traits for the built-in int64 type. */
template<> struct TVariantTraits<int64>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Int64; }
};


/** Implements variant type traits for the built-in FIntPoint type. */
template<> struct TVariantTraits<FIntPoint>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::IntPoint; }
};


/** Implements variant type traits for the built-in FIntVector type. */
template<> struct TVariantTraits<FIntVector>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::IntVector; }
};


/** Implements variant type traits for the built-in FIntRect type. */
template<> struct TVariantTraits<FIntRect>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::IntRect; }
};


/** Implements variant type traits for the built-in FLinearColor type. */
template<> struct TVariantTraits<FLinearColor>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::LinearColor; }
};


/** Implements variant type traits for the built-in FMatrix type. */
template<> struct TVariantTraits<FMatrix>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Matrix; }
};


/** Implements variant type traits for the built-in FPlane type. */
template<> struct TVariantTraits<FPlane>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Plane; }
};


/** Implements variant type traits for the built-in FQuat type. */
template<> struct TVariantTraits<FQuat>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Quat; }
};


/** Implements variant type traits for the built-in FName type. */
template<> struct TVariantTraits<FName>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Name; }
};


/** Implements variant type traits for the built-in FRandomStream type. */
template<> struct TVariantTraits<FRandomStream>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::RandomStream; }
};


/** Implements variant type traits for the built-in FRotator type. */
template<> struct TVariantTraits<FRotator>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Rotator; }
};


/** Implements variant type traits for the built-in FString type. */
template<> struct TVariantTraits<FString>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::String; }
};


/** Implements variant type traits for the built-in WIDECHAR type. */
template<> struct TVariantTraits<WIDECHAR>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Widechar; }
};


/** Implements variant type traits for the built-in FTimespan type. */
template<> struct TVariantTraits<FTimespan>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Timespan; }
};


/** Implements variant type traits for the built-in FTransform type. */
template<> struct TVariantTraits<FTransform>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Transform; }
};


/** Implements variant type traits for the built-in FTwoVectors type. */
template<> struct TVariantTraits<FTwoVectors>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::TwoVectors; }
};


/** Implements variant type traits for the built-in uint8 type. */
template<> struct TVariantTraits<uint8>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::UInt8; }
};


/** Implements variant type traits for the built-in uint16 type. */
template<> struct TVariantTraits<uint16>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::UInt16; }
};


/** Implements variant type traits for the built-in uint32 type. */
template<> struct TVariantTraits<uint32>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::UInt32; }
};


/** Implements variant type traits for the built-in uint64 type. */
template<> struct TVariantTraits<uint64>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::UInt64; }
};


/** Implements variant type traits for the built-in FVector type. */
template<> struct TVariantTraits<FVector>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Vector; }
};


/** Implements variant type traits for the built-in FVector2D type. */
template<> struct TVariantTraits<FVector2D>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Vector2d; }
};


/** Implements variant type traits for the built-in FVector4 type. */
template<> struct TVariantTraits<FVector4>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::Vector4; }
};


/** Implements variant type traits for the built-in NetworkGUID type. */
template<> struct TVariantTraits<FNetworkGUID>
{
	static constexpr EVariantTypes GetType() { return EVariantTypes::NetworkGUID; }
};
