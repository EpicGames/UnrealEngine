// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/CString.h"

/**
 * Implements a string with wild card pattern matching abilities.
 * 
 * The FWildcardString is meant to hold the pattern you are matching against.
 * For basic use, just call the static functions IsMatch() or IsMatchSubstring
 * if you have FStringView
 */
class FWildcardString
	: public FString
{
public:

	/** Default constructor. */
	FWildcardString( )
		: FString()
	{ }

	/**
	 * Creates and initializes a new instance with the specified pattern.
	 *
	 * @param Pattern The pattern string.
	 */
	FWildcardString( const FString& Pattern )
		: FString(Pattern)
	{ }

	/**
	 * Creates and initializes a new instance with the specified pattern.
	 *
	 * @param Pattern The pattern string.
	 */
	FWildcardString( const TCHAR* Pattern )
		: FString(Pattern)
	{ }

public:

	/**
	 * Checks whether this string contains wild card characters.
	 *
	 * @return true if this string contains wild cards, false otherwise.
	 */
	bool ContainsWildcards( ) const
	{
		return ContainsWildcards(**this);
	}

	/**
	 * Matches the given input string to this wild card pattern.
	 *
	 * @param Input The string to match.
	 * @return true if the input string matches this pattern, false otherwise.
	 */
	bool IsMatch( const TCHAR* Input ) const
	{
		return IsMatch(**this, Input);
	}

	/**
	 * Matches the given input string to this wild card pattern.
	 *
	 * @param Input The string to match.
	 * @return true if the input string matches this pattern, false otherwise.
	 */
	bool IsMatch( const FString& Input ) const
	{
		return IsMatch(**this, *Input);
	}

public:

	/**
	 * Checks whether the specified pattern contains wild card characters.
	 *
	 * @param Pattern The string to check.
	 * @return true if the string contains wild cards, false otherwise.
	 */
	static CORE_API bool ContainsWildcards( const TCHAR* Pattern );

	/**
	 * Non-recursive wild card string matching algorithm.
	 *
	 * @param Pattern The pattern to match.
	 * @param Input The input string to check.
	 */
	static CORE_API bool IsMatch( const TCHAR* Pattern, const TCHAR* Input );

	/**
	* 
	* As IsMatch, except can accept the end of the input string in order to facilitate
	* FStringView usage.
	* 
	* if ESearchCase::IgnoreCase is used, the pattern and input are ToLower()d before
	* comparison (note - does not apply locale and just uses c runtime style tolower)
	*/
	static CORE_API bool IsMatchSubstring( const TCHAR* Pattern, const TCHAR* Input, const TCHAR* InputEnd, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);

protected:

	/** Holds the string terminator character. */
	static const TCHAR EndOfString = TCHAR('\0');

	/** Holds the wild card that matches exactly one character (default is '?'). */
	static const TCHAR ExactWildcard = TCHAR('?');

	/** Holds the wild card that matches a sequence of characters (default is '*'). */
	static const TCHAR SequenceWildcard = TCHAR('*');
};
