// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/Integral.h"
#include "Misc/UEOps.h"
#include "Templates/Tuple.h"
#include <iterator>

namespace UE::Core::Private
{
	template <typename IteratorTupleType>
	struct TIteratorTuple;

	template <typename... IteratorTypes>
	struct TIteratorTuple<TTuple<IteratorTypes...>>
	{
		using value_type = TTuple<std::decay_t<decltype(*std::declval<IteratorTypes>())>...>;

		TTuple<IteratorTypes...> Iterators;

		decltype(auto) operator*() const
		{
			return this->Iterators.ApplyAfter(
				[](const auto&... Iters)
				{
					return TTuple<decltype(*Iters)...>(*Iters...);
				}
			);
		}

		TIteratorTuple& operator++()
		{
			VisitTupleElements([](auto& Iters){ ++Iters; }, this->Iterators);
			return *this;
		}

		TIteratorTuple operator++(int)
		{
			TIteratorTuple Result = *this;
			VisitTupleElements([](auto& Iters) { ++Iters; }, this->Iterators);
			return Result;
		}

		TIteratorTuple& operator--()
		{
			VisitTupleElements([](auto& Iters) { --Iters; }, this->Iterators);
			return *this;
		}

		TIteratorTuple operator--(int)
		{
			TIteratorTuple Result = *this;
			VisitTupleElements([](auto& Iters) { --Iters; }, this->Iterators);
			return Result;
		}

		template <UE::CIntegral T>
		[[nodiscard]] TIteratorTuple operator+(T Num) const
		{
			TIteratorTuple Result = *this;
			VisitTupleElements([Num](auto& Iterator) { Iterator += Num; }, Result.Iterators);
			return Result;
		}

		template <UE::CIntegral T>
		[[nodiscard]] TIteratorTuple operator-(T Num) const
		{
			TIteratorTuple Result = *this;
			VisitTupleElements([Num](auto& Iterator) { Iterator -= Num; }, Result.Iterators);
			return Result;
		}

		template <UE::CIntegral T>
		TIteratorTuple& operator+=(T Num)
		{
			VisitTupleElements([Num](auto& Iterator) { Iterator += Num; }, this->Iterators);
			return *this;
		}

		template <UE::CIntegral T>
		TIteratorTuple& operator-=(T Num)
		{
			VisitTupleElements([Num](auto& Iterator) { Iterator -= Num; }, this->Iterators);
			return *this;
		}

		[[nodiscard]] auto operator-(const TIteratorTuple& Rhs) const
		{
			return this->Iterators.template Get<0>() - Rhs.Iterators.template Get<0>();
		}

		template <UE::CIntegral T>
		[[nodiscard]] decltype(auto) operator[](T Index) const
		{
			return *(*this + Index);
		}

		[[nodiscard]] bool UEOpEquals(const TIteratorTuple& Rhs) const
		{
			bool bResult = this->Iterators.template Get<0>() == Rhs.Iterators.template Get<0>();

			// If the first elements compare unequal, all iterators should also compare unequal, or
			// it means the first container is longer than one of the others.
			checkfSlow(bResult || All(*this, Rhs, [](const auto& A, const auto& B) { return !(A == B); }), TEXT("UE::Zip - initial range is longer than other ranges"));

			return bResult;
		}

		[[nodiscard]] bool UEOpLessThan(const TIteratorTuple& Rhs) const
		{
			bool bResult = this->Iterators.template Get<0>() < Rhs.Iterators.template Get<0>();

			// If the first elements compare less-than, all iterators should also compare less-than, or
			// it means the first container is longer than one of the others.
			checkfSlow(!bResult || All(*this, Rhs, [](const auto& A, const auto& B) { return A < B; }), TEXT("UE::Zip - initial range is longer than other ranges"));

			return bResult;
		}

	private:
		template <typename Predicate>
		static bool All(const TIteratorTuple& Lhs, const TIteratorTuple& Rhs, Predicate Pred)
		{
			bool bResult = true;

			VisitTupleElements(
				[&](const auto& LhsIterator, const auto& RhsIterator)
				{
					bResult = bResult && Pred(LhsIterator, RhsIterator);
				},
				Lhs.Iterators,
				Rhs.Iterators
			);

			return bResult;
		}
	};

	template <typename TupleType>
	TIteratorTuple<std::decay_t<TupleType>> MakeIteratorTuple(TupleType&& Tuple)
	{
		return { (TupleType&&)Tuple };
	}

	template <typename TupleType>
	struct TRangeTuple
	{
		TupleType Ranges;

		auto begin() const
		{
			return MakeIteratorTuple(TransformTuple(this->Ranges, [](auto& Range){ return std::begin(Range); }));
		}

		auto end() const
		{
			return MakeIteratorTuple(TransformTuple(this->Ranges, [](auto& Range){ return std::end(Range); }));
		}
	};
}

namespace UE
{
	// Returns a range which allows iteration over multiple ranges in parallel, that is, the first element from each range
	// is returned, followed by the second elmeent from each range, followed by the third etc.  This is similar to std::views::zip.
	//
	// The 'element type' of the zipped range is a tuple of references into each container.
	//
	// The constness of the references into each container will match the constness of the references
	// returned by the original container, and the constness of the tuple is irrelevant.  AsConst can be used
	// with a container (not view) to return const iterators.  UE::Zip supports mixing const and non-const ranges,
	// and will 'lifetime extend' rvalue containers by taking ownership of them.
	//
	// The zipped range is as long as the first range argument, and it is undefined behaviour to have
	// a first range which is longer than any of the others being zipped.
	//
	// Examples:
	//
	// // Iteration:
	// //   { ContainerOfInts[0], AsConst(ContainerOfStrings)[0] }
	// //   { ContainerOfInts[1], AsConst(ContainerOfStrings)[1] }
	// //   { ContainerOfInts[2], AsConst(ContainerOfStrings)[2] }
	// //   ...
	// for (const auto& Pair : UE::Zip(ContainerOfInts, AsConst(ContainerOfStrings)))
	// {
	//     int32&         Int = Pair.Get<0>();
	//     const FString& Str = Pair.Get<1>();
	// }
	//
	// Sort Array1, Array2 and Array3 in parallel by the elements in Array2
	// Algo::SortBy(UE::Zip(Container1, Container2), GetElementByIndex<1>);
	template <typename Range0Type, typename... OtherRangeTypes>
	auto Zip(Range0Type&& Range0, OtherRangeTypes&&... OtherRanges)
	{
		return UE::Core::Private::TRangeTuple{ TTuple<Range0Type, OtherRangeTypes...>((Range0Type&&)Range0, (OtherRangeTypes&&)OtherRanges...) };
	}
}
