// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "UObject/NameTypes.h"
#include "HAL/Memory.h"
// Include the full definition of FVisualizerDebuggingState so visualizers can see the full type information
//   Without this include (and just a fwd decl), visualizers in some modules will not be able to resolve
//   FVisualizerDebuggingState::Ptrs or FVisualizerDebuggingState::GuidString
#include "Modules/VisualizerDebuggingState.h"

// Boilerplate that is included once for each module, even in monolithic builds
#if !defined(PER_MODULE_BOILERPLATE_ANYLINK)
#define PER_MODULE_BOILERPLATE_ANYLINK(ModuleImplClass, ModuleName)
#endif

/**
 * Override new + delete operators (and array versions) in every module.
 * This prevents the possibility of mismatched new/delete calls such as a new[] that
 * uses Unreal's allocator and a delete[] that uses the system allocator.
 *
 * Overloads have to guarantee at least 1 byte is allocated because
 * otherwise new T[0] could return a null pointer, as could ::operator new(0), depending
 * on the allocator (e.g. TBB), which is non-standard behaviour.
 * 
 * FMemory_Malloc, FMemory_Realloc and FMemory_Free have been added for thirdparty
 * libraries that need malloc. These functions will allow for proper memory tracking.
 */
#if USING_CODE_ANALYSIS
	#define OPERATOR_NEW_MSVC_PRAGMA MSVC_PRAGMA( warning( suppress : 28251 ) )	//	warning C28251: Inconsistent annotation for 'new': this instance has no annotations
#else
	#define OPERATOR_NEW_MSVC_PRAGMA
#endif

#define UE_DEFINE_FMEMORY_WRAPPERS \
	void* FMemory_Malloc ( size_t Size, size_t Alignment                 ) { return FMemory::Malloc( Size, Alignment ); } \
	void* FMemory_Realloc( void* Original, size_t Size, size_t Alignment ) { return FMemory::Realloc( Original, Size, Alignment ); } \
	void  FMemory_Free   ( void *Ptr                                     ) { FMemory::Free( Ptr ); }

// Disable the replacement new/delete when running the Clang static analyzer, due to false positives in 15.0.x:
// https://github.com/llvm/llvm-project/issues/58820
#if !FORCE_ANSI_ALLOCATOR && !defined(__clang_analyzer__)
	static_assert(__STDCPP_DEFAULT_NEW_ALIGNMENT__ <= 16, "Expecting 16-byte default operator new alignment - alignments > 16 may have bloat");
	#define REPLACEMENT_OPERATOR_NEW_AND_DELETE \
		FORCENOINLINE OPERATOR_NEW_MSVC_PRAGMA void* operator new  ( size_t Size                                                    ) OPERATOR_NEW_THROW_SPEC      { return FMemory_MallocInline( Size ? Size : 1, Size <= 8 ? 8 : __STDCPP_DEFAULT_NEW_ALIGNMENT__ ); } \
		FORCENOINLINE OPERATOR_NEW_MSVC_PRAGMA void* operator new[]( size_t Size                                                    ) OPERATOR_NEW_THROW_SPEC      { return FMemory_MallocInline( Size ? Size : 1, Size <= 8 ? 8 : __STDCPP_DEFAULT_NEW_ALIGNMENT__ ); } \
		FORCENOINLINE OPERATOR_NEW_MSVC_PRAGMA void* operator new  ( size_t Size,                             const std::nothrow_t& ) OPERATOR_NEW_NOTHROW_SPEC    { return FMemory_MallocInline( Size ? Size : 1, Size <= 8 ? 8 : __STDCPP_DEFAULT_NEW_ALIGNMENT__ ); } \
		FORCENOINLINE OPERATOR_NEW_MSVC_PRAGMA void* operator new[]( size_t Size,                             const std::nothrow_t& ) OPERATOR_NEW_NOTHROW_SPEC    { return FMemory_MallocInline( Size ? Size : 1, Size <= 8 ? 8 : __STDCPP_DEFAULT_NEW_ALIGNMENT__ ); } \
		FORCENOINLINE OPERATOR_NEW_MSVC_PRAGMA void* operator new  ( size_t Size, std::align_val_t Alignment                        ) OPERATOR_NEW_THROW_SPEC      { return FMemory_MallocInline( Size ? Size : 1, (std::size_t)Alignment ); } \
		FORCENOINLINE OPERATOR_NEW_MSVC_PRAGMA void* operator new[]( size_t Size, std::align_val_t Alignment                        ) OPERATOR_NEW_THROW_SPEC      { return FMemory_MallocInline( Size ? Size : 1, (std::size_t)Alignment ); } \
		FORCENOINLINE OPERATOR_NEW_MSVC_PRAGMA void* operator new  ( size_t Size, std::align_val_t Alignment, const std::nothrow_t& ) OPERATOR_NEW_NOTHROW_SPEC    { return FMemory_MallocInline( Size ? Size : 1, (std::size_t)Alignment ); } \
		FORCENOINLINE OPERATOR_NEW_MSVC_PRAGMA void* operator new[]( size_t Size, std::align_val_t Alignment, const std::nothrow_t& ) OPERATOR_NEW_NOTHROW_SPEC    { return FMemory_MallocInline( Size ? Size : 1, (std::size_t)Alignment ); } \
		FORCENOINLINE void operator delete  ( void* Ptr                                                                             ) OPERATOR_DELETE_THROW_SPEC   { FMemory_FreeInline( Ptr ); } \
		FORCENOINLINE void operator delete[]( void* Ptr                                                                             ) OPERATOR_DELETE_THROW_SPEC   { FMemory_FreeInline( Ptr ); } \
		FORCENOINLINE void operator delete  ( void* Ptr,                                                      const std::nothrow_t& ) OPERATOR_DELETE_NOTHROW_SPEC { FMemory_FreeInline( Ptr ); } \
		FORCENOINLINE void operator delete[]( void* Ptr,                                                      const std::nothrow_t& ) OPERATOR_DELETE_NOTHROW_SPEC { FMemory_FreeInline( Ptr ); } \
		FORCENOINLINE void operator delete  ( void* Ptr,             size_t Size                                                    ) OPERATOR_DELETE_THROW_SPEC   { FMemory_FreeInline( Ptr ); } \
		FORCENOINLINE void operator delete[]( void* Ptr,             size_t Size                                                    ) OPERATOR_DELETE_THROW_SPEC   { FMemory_FreeInline( Ptr ); } \
		FORCENOINLINE void operator delete  ( void* Ptr,             size_t Size,                             const std::nothrow_t& ) OPERATOR_DELETE_NOTHROW_SPEC { FMemory_FreeInline( Ptr ); } \
		FORCENOINLINE void operator delete[]( void* Ptr,             size_t Size,                             const std::nothrow_t& ) OPERATOR_DELETE_NOTHROW_SPEC { FMemory_FreeInline( Ptr ); } \
		FORCENOINLINE void operator delete  ( void* Ptr,                          std::align_val_t Alignment                        ) OPERATOR_DELETE_THROW_SPEC   { FMemory_FreeInline( Ptr ); } \
		FORCENOINLINE void operator delete[]( void* Ptr,                          std::align_val_t Alignment                        ) OPERATOR_DELETE_THROW_SPEC   { FMemory_FreeInline( Ptr ); } \
		FORCENOINLINE void operator delete  ( void* Ptr,                          std::align_val_t Alignment, const std::nothrow_t& ) OPERATOR_DELETE_NOTHROW_SPEC { FMemory_FreeInline( Ptr ); } \
		FORCENOINLINE void operator delete[]( void* Ptr,                          std::align_val_t Alignment, const std::nothrow_t& ) OPERATOR_DELETE_NOTHROW_SPEC { FMemory_FreeInline( Ptr ); } \
		FORCENOINLINE void operator delete  ( void* Ptr,             size_t Size, std::align_val_t Alignment                        ) OPERATOR_DELETE_THROW_SPEC   { FMemory_FreeInline( Ptr ); } \
		FORCENOINLINE void operator delete[]( void* Ptr,             size_t Size, std::align_val_t Alignment                        ) OPERATOR_DELETE_THROW_SPEC   { FMemory_FreeInline( Ptr ); } \
		FORCENOINLINE void operator delete  ( void* Ptr,             size_t Size, std::align_val_t Alignment, const std::nothrow_t& ) OPERATOR_DELETE_NOTHROW_SPEC { FMemory_FreeInline( Ptr ); } \
		FORCENOINLINE void operator delete[]( void* Ptr,             size_t Size, std::align_val_t Alignment, const std::nothrow_t& ) OPERATOR_DELETE_NOTHROW_SPEC { FMemory_FreeInline( Ptr ); }
#else
	#define REPLACEMENT_OPERATOR_NEW_AND_DELETE
#endif

class FChunkedFixedUObjectArray;

// Visualizer macro to prevent dead-stripping of global debug state variables.
// This has not (yet) been an issue under MSVC, which is missing support for used and retain.
#if defined(__has_cpp_attribute)
	#if __has_cpp_attribute(used) && __has_cpp_attribute(retain)
		#define UE_VISUALIZER_USE_AND_RETAIN [[used]] [[retain]]			// __GNUC__
	#elif __has_cpp_attribute(gnu::used) && __has_cpp_attribute(gnu::retain)
		#define UE_VISUALIZER_USE_AND_RETAIN [[gnu::used]] [[gnu::retain]]	// __clang__
	#endif
#endif
#ifndef UE_VISUALIZER_USE_AND_RETAIN
#define UE_VISUALIZER_USE_AND_RETAIN
#endif

#ifdef DISABLE_UE4_VISUALIZER_HELPERS
	UE_DEPRECATED_HEADER(5.5, "DISABLE_UE4_VISUALIZER_HELPERS has been disabled, please define UE_ENABLE_VISUALIZER_HELPERS=0 instead.")
	#define UE_ENABLE_VISUALIZER_HELPERS 0
#endif
#ifndef UE_ENABLE_VISUALIZER_HELPERS
	#define UE_ENABLE_VISUALIZER_HELPERS 1
#endif

#if !UE_ENABLE_VISUALIZER_HELPERS
	#define UE_VISUALIZERS_HELPERS
#elif PLATFORM_UNIX || PLATFORM_MAC
	// GDB/LLDB pretty printers don't use these - no need to export additional symbols. This also solves ODR violation reported by ASan on Linux
	#define UE_VISUALIZERS_HELPERS
#else
	#define UE_VISUALIZERS_HELPERS \
		UE_VISUALIZER_USE_AND_RETAIN uint8** GNameBlocksDebug = FNameDebugVisualizer(FClangKeepDebugInfo{}).GetBlocks(); \
		UE_VISUALIZER_USE_AND_RETAIN FChunkedFixedUObjectArray*& GObjectArrayForDebugVisualizers = GCoreObjectArrayForDebugVisualizers; \
		UE_VISUALIZER_USE_AND_RETAIN UE::CoreUObject::Private::FStoredObjectPathDebug*& GComplexObjectPathDebug = GCoreComplexObjectPathDebug; \
		UE_VISUALIZER_USE_AND_RETAIN UE::CoreUObject::Private::FObjectHandlePackageDebugData*& GObjectHandlePackageDebug = GCoreObjectHandlePackageDebug; \
		UE_VISUALIZER_USE_AND_RETAIN UE::Core::FVisualizerDebuggingState*& GDebuggingState = GCoreDebuggingState; \
		UE_VISUALIZER_USE_AND_RETAIN UE::CoreUObject::Private::FStoreBase* const*& GFixedTagStoresForDebugVisualizers = GCoreFixedTagStoresForDebugVisualizers;
#endif

// in DLL builds, these are done per-module, otherwise we just need one in the application
// visual studio cannot find cross dll data for visualizers, so these provide access
#define PER_MODULE_BOILERPLATE \
	UE_VISUALIZERS_HELPERS \
	REPLACEMENT_OPERATOR_NEW_AND_DELETE \
	UE_DEFINE_FMEMORY_WRAPPERS
