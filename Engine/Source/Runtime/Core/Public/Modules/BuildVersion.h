// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/EngineVersion.h"

class FJsonObject;

/**
 * Stores the version information associated with a build
 */
class FBuildVersion
{
public:
	/**
	 * The major engine version (4 for UE4)
	 */
	int MajorVersion;

	/**
	 * The minor engine version
	 */
	int MinorVersion;

	/**
	 * The hotfix/patch version
	 */
	int PatchVersion;

	/**
	 * The changelist that the engine is being built from
	 */
	int Changelist;

	/**
	 * The changelist that the engine maintains compatibility with
	 */
	int CompatibleChangelist;

	/**
	 * Whether the changelist numbers are a licensee changelist
	 */
	int IsLicenseeVersion;

	/**
	 * Whether the current build is a promoted build, that is, built strictly from a clean sync of the given changelist
	 */
	int IsPromotedBuild;

	/**
	 * Name of the current branch, with '/' characters escaped as '+'
	 */
	FString BranchName;

	/**
	 * The current build id. This will be generated automatically whenever engine binaries change if not set in the default Engine/Build/Build.version.
	 */
	FString BuildId;

	/**
	 * The build version string.
	 */
	FString BuildVersion;

	/**
	 * [Optional] A url to the web page for the build (if it was created by an automated build systme for example)
	 */
	FString BuildUrl;

	/**
	 * Default constructor. Initializes the structure to empty.
	 */
	CORE_API FBuildVersion();

	/// <summary>
	/// Gets the compatible changelist if set, otherwise the default compatible changelist
	/// </summary>
	/// <returns>The compatible changelist</returns>
	CORE_API int GetEffectiveCompatibleChangelist() const;

	/// <summary>
	/// Get an engine version object for this build version
	/// </summary>
	/// <returns>New engine version object</returns>
	CORE_API FEngineVersion GetEngineVersion() const;

	/// <summary>
	/// Get a compatible engine version object for this build version
	/// </summary>
	/// <returns>New engine version object</returns>
	CORE_API FEngineVersion GetCompatibleEngineVersion() const;

	/// <summary>
	/// Get the default path to the build.version file on disk
	/// </summary>
	/// <returns>Path to the Build.version file</returns>
	static CORE_API FString GetDefaultFileName();

	/// <summary>
	/// Get the path to the version file for the current executable.
	/// </summary>
	/// <returns>Path to the target's version file</returns>
	static CORE_API FString GetFileNameForCurrentExecutable();
	
	/**
	 * Try to read a version file from disk
	 *
	 * @param FileName		Path to the version file
	 * @param OutVersion	The version information
	 * @return				True if the version was read successfully, false otherwise
	 */
	static CORE_API bool TryRead(const FString& FileName, FBuildVersion& OutVersion);

	/**
	 * Try to read version info from data stored in a string.
	 * 
	 * @param Text			The version info stored in string form
	 * @param OutVersion	The version information
	 * @return				True if the version was read successfully, false otherwise
	 */
	static CORE_API bool TryReadFromString(const FString& Text, FBuildVersion& OutVersion);
};
