// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Interface class that all module implementations should derive from. This is used to initialize a module after it's
 * been loaded, and also to clean it up before the module is unloaded. Callbacks are invoked at the following times:
 *
 * Operation                                         | Startup | PostLoad | PreUnload | Shutdown |
 * --------------------------------------------------|---------|----------|-----------|----------|
 * Engine Startup                                    | Yes     | No       | ----      | ----     |
 * Engine Shutdown                                   | ----    | ----     | Yes       | Yes      |
 * Hot Reload (Legacy)                               | Yes     | Yes      | Yes       | Yes      |
 * Live Coding                                       | No      | No       | No        | No       |
 * CCmd Load                                         | Yes     | Yes      | ----      | ----     |
 * CCmd Unload                                       | ----    | ----     | Yes       | Yes      |
 * CCmd Reload                                       | Yes     | Yes      | Yes       | Yes      |
 * FModuleManager::LoadModule                        | Yes     | No       | ----      | ----     |
 * FModuleManager::LoadModuleChecked                 | Yes     | No       | ----      | ----     |
 * FModuleManager::LoadModuleWithCallback            | Yes     | Yes      | ----      | ----     |
 * FModuleManager::LoadModulePtr                     | Yes     | No       | ----      | ----     |
 * FModuleManager::LoadModuleWithFailureReason       | Yes     | No       | ----      | ----     |
 * FModuleManager::UnloadModule                      | ----    | ----     | No        | Yes      |
 * FModuleManager::UnloadOrAbandonModuleWithCallback | ----    | ----     | Yes       | Yes      |
 * FModuleManager::AbandonModule                     | ----    | ----     | No        | Yes      |
 * FModuleManager::AbandonModuleWithCallback         | ----    | ----     | Yes       | Yes      |
 */
class IModuleInterface
{
public:

	/**
	 * Note: Even though this is an interface class we need a virtual destructor here because modules are deleted via a
	 * pointer to this interface.
	 */
	virtual ~IModuleInterface() = default;

	/**
	 * Called after the module is loaded. Occurs in all loading situations, including engine startup, loading or
	 * reloading via console commands, hot reloading, and explicit requests through @ref FModuleManager public API.
	 * @ref FModuleManager::CurrentModule is guaranteed to be set when this is called.
	 *
	 * Load dependent modules here, and they will be guaranteed to be available during ShutdownModule. ie:
	 *
	 * FModuleManager::Get().LoadModuleChecked(TEXT("HTTP"));
	 */
	virtual void StartupModule()
	{
	}

	/**
	 * Called before the module is unloaded. Occurs during engine shutdown, hot reloading, and unloading or reloading
	 * via console commands. During engine shutdown this is called for all modules before ShutdownModule is called for
	 * any module. During engine shutdown, this is called in reverse order that modules finish StartupModule. Is not
	 * called for explicit @ref FModuleManager::UnloadModule and @ref FModuleManager::AbandonModule requests.
	 */
	virtual void PreUnloadCallback()
	{
	}

	/**
	 * Called after the module has been reloaded. Occurs during hot reloading and loading or reloading via console
	 * commands. Not called during engine startup or explicit requests through @ref FModuleManager public API, other
	 * than @ref FModuleManager::LoadModuleWithCallback.
	 */
	virtual void PostLoadCallback()
	{
	}

	/**
	 * Called before the module is unloaded. Occurs in all unloading situations, including hot reloading, unloading or
	 * reloading via console commands, explicit requests through @ref FModuleManager public API, and engine shutdown.
	 * During engine shutdown, this is called in reverse order that modules finish StartupModule. This means that, as
	 * long as a module references dependent modules in it's StartupModule, it can safely reference those dependencies
	 * in ShutdownModule as well.
	 */
	virtual void ShutdownModule()
	{
	}

	/**
	 * Override this to set whether your module is allowed to be unloaded on the fly
	 *
	 * @return Whether the module supports shutdown separate from the rest of the engine.
	 */
	virtual bool SupportsDynamicReloading()
	{
		return true;
	}

	/**
	 * Override this to set whether your module would like cleanup on application shutdown
	 *
	 * @return Whether the module supports shutdown on application exit
	 */
	virtual bool SupportsAutomaticShutdown()
	{
		return true;
	}

	/**
	 * Returns true if this module hosts gameplay code
	 *
	 * @return True for "gameplay modules", or false for engine code modules, plugins, etc.
	 */
	virtual bool IsGameModule() const
	{
		return false;
	}
};
