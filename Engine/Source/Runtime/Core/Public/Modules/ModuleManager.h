// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/Defines.h"
#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PreprocessorHelpers.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Build.h"
#include "Misc/CoreMisc.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "Modules/Boilerplate/ModuleBoilerplate.h"
#include "Modules/ModuleInterface.h"
#include "Serialization/Archive.h"
#include "Templates/Atomic.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

class FArchive;
class FModuleManager;
class FOutputDevice;
class UClass;
class UWorld;

#if WITH_ENGINE
	extern CORE_API TMap<UClass*, UClass*>& GetClassesToReinstanceForHotReload(); 
#endif


/**
 * Enumerates reasons for failed module loads.
 */
enum class EModuleLoadResult
{
	/** Module loaded successfully. */
	Success,

	/** The specified module file could not be found. */
	FileNotFound,

	/** The specified module file is incompatible with the module system. */
	FileIncompatible,

	/** The operating system failed to load the module file. */
	CouldNotBeLoadedByOS,

	/** Module initialization failed. */
	FailedToInitialize,

	/** A thread attempted to load the module before the Game thread did. */
	NotLoadedByGameThread
};

CORE_API const TCHAR* LexToString(EModuleLoadResult LoadResult);

/**
 * Enumerates reasons for failed module unloads.
 */
enum class EModuleUnloadResult
{
	/** Module unloaded successfully. */
	Success,

	/** Module does not support dynamic reloading (see IModuleInterface::SupportsDynamicReloading). */
	UnloadNotSupported
};

/**
 * Enumerates reasons for modules to change.
 *
 * Values of this type will be passed into OnModuleChanged() delegates.
 */
enum class EModuleChangeReason
{
	/** A module has been loaded and is ready to be used. */
	ModuleLoaded,

	/* A module has been unloaded and should no longer be used. */
	ModuleUnloaded,

	/** The paths controlling which plug-ins are loaded have been changed and the given module has been found, but not yet loaded. */
	PluginDirectoryChanged
};


enum class ECheckModuleCompatibilityFlags
{
	None = 0x00,

	// Display the loading of an up-to-date module
	DisplayUpToDateModules = 0x01
};

ENUM_CLASS_FLAGS(ECheckModuleCompatibilityFlags)


enum class ELoadModuleFlags
{
	None = 0x0,

	// Print to the log any failure information
	LogFailures = 1 << 0,
};

ENUM_CLASS_FLAGS(ELoadModuleFlags)


/**
 * Structure for reporting module statuses.
 */
struct FModuleStatus
{
	/** Default constructor. */
	FModuleStatus()
		: bIsLoaded(false)
		, bIsGameModule(false)
	{ }

	/** Short name for this module. */
	FString Name;

	/** Full path to this module file on disk. */
	FString FilePath;

	/** Whether the module is currently loaded or not. */
	bool bIsLoaded;

	/** Whether this module contains game play code. */
	bool bIsGameModule;
};

/**
 * Structure for reporting module disk presence info.
 */
struct FModuleDiskInfo
{
	/** Short name for this module. */
	FName Name;

	/** Full path to this module file on disk. */
	FString FilePath;
};

namespace UE::Core::Private
{
	TOptional<FModuleManager>& GetModuleManagerSingleton();
}

/**
 * The module manager is used to load and unload modules, as well as to keep track of all of the
 * modules that are currently loaded. You can access this singleton using FModuleManager::Get().
 */
class FModuleManager
	: private FSelfRegisteringExec
{
public:

	/**
	 * Destructor.
	 */
	CORE_API ~FModuleManager();


	/**
	 * Gets the singleton instance of the module manager.
	 *
	 * @return The module manager instance.
	 */
	static CORE_API FModuleManager& Get( );

	/** Destroys singleton if it exists. Get() must not be called after Destroy(). */
	static CORE_API void TearDown();


	/**
	 * Abandons a loaded module, leaving it loaded in memory but no longer tracking it in the module manager.
	 * Does not call @ref IModuleInterface::PreUnloadCallback. Calls @ref IModuleInterface::ShutdownModule.
	 *
	 * @param InModuleName The name of the module to abandon.  Should not include path, extension or platform/configuration info.  This is just the "module name" part of the module file name.
	 * @see IsModuleLoaded, LoadModule, LoadModuleWithFailureReason, UnloadModule
	 */
	CORE_API void AbandonModule( const FName InModuleName );

	/**
	 * Adds a module to our list of modules, unless it's already known.
	 *
	 * @param InModuleName The base name of the module file.  Should not include path, extension or platform/configuration info.  This is just the "name" part of the module file name.  Names should be globally unique.
	 */
	CORE_API void AddModule( const FName InModuleName );

#if !IS_MONOLITHIC
	CORE_API void RefreshModuleFilenameFromManifest(const FName InModuleName);
#endif	// !IS_MONOLITHIC

	/**
	 * Gets the specified module.
	 *
	 * @param InModuleName Name of the module to return.
	 * @return 	The module, or nullptr if the module is not loaded.
	 * @see GetModuleChecked, GetModulePtr
	 */
	CORE_API IModuleInterface* GetModule( const FName InModuleName );

	/**
	 * Checks whether a module is compiled-in, or currently loaded, with all code inside safe to call.
	 * 
	 * @param InModuleName Name of the module to check.
	 * @return true for a module that exists and is not compiled into the executable.
	 */
	CORE_API bool IsModuleSafeToUse( const FName InModuleName ) const;

	/**
	 * Checks whether the specified module is currently loaded.
	 *
	 * This is an O(1) operation.
	 *
	 * @param InModuleName The base name of the module file.  Should not include path, extension or platform/configuration info.  This is just the "module name" part of the module file name.  Names should be globally unique.
	 * @return true if module is currently loaded, false otherwise.
	 * @see AbandonModule, LoadModule, LoadModuleWithFailureReason, UnloadModule
	 */
	CORE_API bool IsModuleLoaded( const FName InModuleName ) const;

	/**
	 * Loads the specified module.
	 * Calls @ref IModuleInterface::StartupModule. Does not call @ref IModuleInterface::PostLoadCallback.
	 *
	 * @param InModuleName The base name of the module file.  Should not include path, extension or platform/configuration info.  This is just the "module name" part of the module file name.  Names should be globally unique.
	 * @param InLoadModuleFlags Optional flags for module load operation.
	 * @return The loaded module, or nullptr if the load operation failed.
	 * @see AbandonModule, IsModuleLoaded, LoadModuleChecked, LoadModulePtr, LoadModuleWithFailureReason, UnloadModule
	 */
	CORE_API IModuleInterface* LoadModule( const FName InModuleName, ELoadModuleFlags InLoadModuleFlags = ELoadModuleFlags::None );

	/**
	 * Loads the specified module, checking to ensure it exists.
	 *
	 * @param InModuleName The base name of the module file.  Should not include path, extension or platform/configuration info.  This is just the "module name" part of the module file name.  Names should be globally unique.
	 * @return The loaded module.
	 * @see AbandonModule, IsModuleLoaded, LoadModuleChecked, LoadModulePtr, LoadModuleWithFailureReason, UnloadModule
	 */
	CORE_API IModuleInterface& LoadModuleChecked( const FName InModuleName );

	/**
	 * Loads a module in memory then calls PostLoad.
	 * Calls @ref IModuleInterface::StartupModule and @ref IModuleInterface::PostLoadCallback.
	 *
	 * @param InModuleName The name of the module to load.
	 * @param Ar The archive to receive error messages, if any.
	 * @return true on success, false otherwise.
	 * @see UnloadOrAbandonModuleWithCallback
	 */
	CORE_API bool LoadModuleWithCallback( const FName InModuleName, FOutputDevice &Ar );

	/**
	 * Loads the specified module and returns a result.
	 * Calls @ref IModuleInterface::StartupModule. Does not call @ref IModuleInterface::PostLoadCallback.
	 *
	 * @param InModuleName The base name of the module file.  Should not include path, extension or platform/configuration info.  This is just the "module name" part of the module file name.  Names should be globally unique.
	 * @param OutFailureReason Will contain the result.
	 * @param InLoadModuleFlags Optional flags for module load operation.
	 * @return The loaded module (null if the load operation failed).
	 * @see AbandonModule, IsModuleLoaded, LoadModule, LoadModuleChecked, LoadModulePtr, UnloadModule
	 */
	CORE_API IModuleInterface* LoadModuleWithFailureReason( const FName InModuleName, EModuleLoadResult& OutFailureReason, ELoadModuleFlags InLoadModuleFlags = ELoadModuleFlags::None);

	/**
	 * Queries information about a specific module name.
	 *
	 * @param InModuleName Module to query status for.
	 * @param OutModuleStatus Status of the specified module.
	 * @return true if the module was found and the OutModuleStatus is valid, false otherwise.
	 * @see QueryModules
	 */
	CORE_API bool QueryModule( const FName InModuleName, FModuleStatus& OutModuleStatus ) const;

	/**
	 * Queries information about all of the currently known modules.
	 *
	 * @param OutModuleStatuses Status of all modules. Note that the array will be cleared before the results are added to it.
	 * @see QueryModule
	 */
	CORE_API void QueryModules( TArray<FModuleStatus>& OutModuleStatuses ) const;

	/**
	 * Unloads a specific module
	 * Does not call @ref IModuleInterface::PreUnloadCallback. Calls @ref IModuleInterface::ShutdownModule.
	 * NOTE: You can manually unload a module before the normal shutdown occurs with this, but be careful as you may be unloading another
	 * module's dependency too early!
	 *
	 * @param InModuleName The name of the module to unload.  Should not include path, extension or platform/configuration info.  This is just the "module name" part of the module file name.
	 * @param bIsShutdown Is this unload module call occurring at shutdown (default = false).
	 * @param bAllowUnloadCode Allow unloading of code library if possible (default = true).
	 * @return true if module was unloaded successfully, false otherwise.
	 * @see AbandonModule, IsModuleLoaded, LoadModule, LoadModuleWithFailureReason
	 */
	CORE_API bool UnloadModule( const FName InModuleName, bool bIsShutdown = false, bool bAllowUnloadCode = true );

	/**
	 * Calls PreUnload then either unloads or abandons a module in memory, depending on whether the module supports unloading.
	 * Calls @ref IModuleInterface::PreUnloadCallback and @ref IModuleInterface::ShutdownModule.
	 *
	 * @param InModuleName The name of the module to unload.
	 * @param Ar The archive to receive error messages, if any.
	 * @see LoadModuleWithCallback
	 */
	CORE_API void UnloadOrAbandonModuleWithCallback( const FName InModuleName, FOutputDevice &Ar);

	/**
	 * Calls PreUnload then abandons a module in memory.
	 * Calls @ref IModuleInterface::PreUnloadCallback and @ref IModuleInterface::ShutdownModule.
	 *
	 * @param InModuleName The name of the module to unload.
	 * @see LoadModuleWithCallback
	 */
	CORE_API void AbandonModuleWithCallback( const FName InModuleName );

	/**
	 * Add any extra search paths that may be required
	 */
	CORE_API void AddExtraBinarySearchPaths();

	/**
	  * Gets a module by name, checking to ensure it exists.
	  *
	  * This method checks whether the module actually exists. If the module does not exist, an assertion will be triggered.
	  *
	  * @param ModuleName The module to get.
	  * @return The interface to the module.
	  * @see GetModulePtr, LoadModulePtr, LoadModuleChecked
	  */
	template<typename TModuleInterface>
	static TModuleInterface& GetModuleChecked( const FName ModuleName )
	{
		FModuleManager& ModuleManager = FModuleManager::Get();

		checkf(ModuleManager.IsModuleLoaded(ModuleName), TEXT("Tried to get module interface for unloaded module: '%s'"), *(ModuleName.ToString()));
		return static_cast<TModuleInterface&>(*ModuleManager.GetModule(ModuleName));
	}

private:
	static CORE_API IModuleInterface* GetModulePtr_Internal(FName ModuleName);

	IModuleInterface* GetOrLoadModule(const FName InModuleName, EModuleLoadResult& OutFailureReason, ELoadModuleFlags InLoadModuleFlags = ELoadModuleFlags::None);

public:

	/**
	  * Gets a module by name.
	  *
	  * @param ModuleName The module to get.
	  * @return The interface to the module, or nullptr if the module was not found.
	  * @see GetModuleChecked, LoadModulePtr, LoadModuleChecked
	  */
	template<typename TModuleInterface>
	static UE_FORCEINLINE_HINT TModuleInterface* GetModulePtr( const FName ModuleName )
	{
		return static_cast<TModuleInterface*>(GetModulePtr_Internal(ModuleName));
	}

	/**
	 * Loads a module by name, checking to ensure it exists.
	 * Calls @ref IModuleInterface::StartupModule. Does not call @ref IModuleInterface::PostLoadCallback.
	 *
	 * This method checks whether the module actually exists. If the module does not exist, an assertion will be triggered.
	 * If the module was already loaded previously, the existing instance will be returned.
	 *
	 * @param ModuleName The module to find and load
	 * @return Returns the module interface, casted to the specified typename
	 * @see GetModulePtr, LoadModulePtr, LoadModuleChecked
	 */
	template<typename TModuleInterface>
	static TModuleInterface& LoadModuleChecked( const FName InModuleName)
	{
		IModuleInterface& ModuleInterface = FModuleManager::Get().LoadModuleChecked(InModuleName);
		return static_cast<TModuleInterface&>(ModuleInterface);
	}

	/**
	 * Loads a module by name.
	 * Calls @ref IModuleInterface::StartupModule. Does not call @ref IModuleInterface::PostLoadCallback.
	 *
	 * @param ModuleName The module to find and load.
	 * @return The interface to the module, or nullptr if the module was not found.
	 * @see GetModulePtr, GetModuleChecked, LoadModuleChecked
	 */
	template<typename TModuleInterface>
	static TModuleInterface* LoadModulePtr( const FName InModuleName)
	{
		return static_cast<TModuleInterface*>(FModuleManager::Get().LoadModule(InModuleName));
	}

	/**
	 * Finds module files on the disk for loadable modules matching the specified wildcard.
	 *
	 * @param WildcardWithoutExtension Filename part (no path, no extension, no build config info) to search for.
	 * @param OutModules List of modules found.
	 */
	CORE_API void FindModules(const TCHAR* WildcardWithoutExtension, TArray<FName>& OutModules) const;
	CORE_API void FindModules(const TCHAR* WildcardWithoutExtension, TArray<FModuleDiskInfo>& OutModules) const;

	/**
	 * Determines if a module with the given name exists, regardless of whether it is currently loaded.
	 *
	 * @param ModuleName Name of the module to look for.
	 * @param OutModuleFilePath If non-null, the assembly filename of the module will be written. 
	 *        Empty string will be written if ModuleExists returns false or assembly is unknown due to e.g.
	 *        Monolithic executable.
	 * @return Whether the module exists.
	 */
	CORE_API bool ModuleExists(const TCHAR* ModuleName, FString* OutModuleFilePath = nullptr) const;

	/**
	 * Gets the number of loaded modules.
	 *
	 * @return The number of modules.
	 */
	CORE_API int32 GetModuleCount( ) const;

	/**
	 * Unloads modules during the shutdown process. Modules are unloaded in reverse order to when their StartupModule() FINISHES.
	 * The practical implication of this is that if module A depends on another module B, and A loads B during A's StartupModule, 
	 * that B will actually get Unloaded after A during shutdown. This allows A's ShutdownModule() call to still reference module B.
	 * You can manually unload a module yourself which will change this ordering, but be careful as you may be unloading another 
	 * module's dependency!
	 * Calls @ref IModuleInterface::PreUnloadCallback and @ref IModuleInterface::ShutdownModule.
	 *
	 * This method is Usually called at various points while exiting an application.
	 */
	CORE_API void UnloadModulesAtShutdown( );


	/** Delegate that's used by the module manager to initialize a registered module that we statically linked with (monolithic only) */
	DECLARE_DELEGATE_RetVal( IModuleInterface*, FInitializeStaticallyLinkedModule )

	/** Raw function type used during module registration. */
	typedef IModuleInterface *FInitializeStaticallyLinkedModuleRaw();

	/**
	 * Registers an initializer for a module that is statically linked.
	 *
	 * @param InModuleName The name of this module.
	 * @param InInitializerDelegate The delegate that will be called to initialize an instance of this module.
	 */
	static CORE_API void RegisterStaticallyLinkedModule( const FLazyName InModuleName, FInitializeStaticallyLinkedModuleRaw* InInitializer );

	/**
	 * Called by the engine at startup to let the Module Manager know that it's now
	 * safe to process new UObjects discovered by loading C++ modules.
	 */
	CORE_API void StartProcessingNewlyLoadedObjects();

	/** Adds an engine binaries directory. */
	CORE_API void AddBinariesDirectory(const TCHAR *InDirectory, bool bIsGameDirectory);

	/** Will load the binary without doing initialization.
	 *  Calling this will not make it possible to unload the module again
	 */
	CORE_API void LoadModuleBinaryOnly(FName ModuleName);

	/**
	 *	Set the game binaries directory
	 *
	 *	@param InDirectory The game binaries directory.
	 */
	CORE_API void SetGameBinariesDirectory(const TCHAR* InDirectory);

	/**
	*	Gets the game binaries directory
	*/
	CORE_API FString GetGameBinariesDirectory() const;

#if !IS_MONOLITHIC
	/**
	 * Checks to see if the specified module exists and is compatible with the current engine version. 
	 *
	 * @param InModuleName The base name of the module file.
	 * @return true if module exists and is up to date, false otherwise.
	 */
	CORE_API bool IsModuleUpToDate( const FName InModuleName ) const;
#endif

	/**
	 * Determines whether the specified module contains UObjects.  The module must already be loaded into
	 * memory before calling this function.
	 *
	 * @param ModuleName Name of the loaded module to check.
	 * @return True if the module was found to contain UObjects, or false if it did not (or wasn't loaded.)
	 */
	CORE_API bool DoesLoadedModuleHaveUObjects( const FName ModuleName ) const;

	/**
	 * Gets the build configuration for compiling modules, as required by UBT.
	 *
	 * @return	Configuration name for UBT.
	 */
	static CORE_API const TCHAR* GetUBTConfiguration( );

#if !IS_MONOLITHIC
	/** Gets the filename for a module. The return value is a full path of a module known to the module manager. */
	CORE_API FString GetModuleFilename(FName ModuleName) const;

	/** Sets the filename for a module. The module is not reloaded immediately, but the new name will be used for subsequent unload/load events. */
	CORE_API void SetModuleFilename(FName ModuleName, const FString& Filename);

	/** Determines if any non-default module instances are loaded (eg. hot reloaded modules) */
	CORE_API bool HasAnyOverridenModuleFilename() const;

	/** Save the current module manager's state into a file for bootstrapping other processes. */
	CORE_API void SaveCurrentStateForBootstrap(const TCHAR* Filename);
#endif

	/**
	 * Gets a multicast delegate that is executed when the set of known modules changed, i.e. upon module load or unload.
	 * The delegate is thread-safe to allow subscribing from other than the game thread but is always broadcasted from the game thread.
	 *
	 * The first parameter is the name of the module that changed.
	 * The second parameter is the reason for the change.
	 *
	 * @return The multicast delegate.
	 */
	TTSMulticastDelegateRegistration<void(FName ModuleName, EModuleChangeReason ChangeReason)>& OnModulesChanged( )
	{
		return ModulesChangedEvent;
	}

	/**
	 * Gets a multicast delegate that is executed when any UObjects need processing after a module was loaded.
	 * Do not use this delegate if you wish to monitor the registration of UObjects for a module. 
	 * Use either CompiledInUObjectsRegisteredDelegate in CoreUObject to reliably receive notifications after UObjects 
	 * have been registered.
	 *
	 * @return The delegate.
	 */
	TMulticastDelegateRegistration<void(FName, bool)>& OnProcessLoadedObjectsCallback()
	{
		return ProcessLoadedObjectsCallback;
	}

	/**
	 * Gets a multicast delegate that is executed to trigger the removal of all UObjects belonging to a list of modules, when these modules are unloaded.
	 * This event only occurs with merged modules, when a merged library is being unmounted.
	 *
	 * @return The delegate.
	 */
	TMulticastDelegateRegistration<void(TConstArrayView<FName>)>& OnModulesUnloaded()
	{
		return OnModulesUnloadCallback;
	}

	/**
	 * Gets a multicast delegate that is executed right before a merged library is unmounted, after every object in linked modules has been destroyed.
	 * This event only occurs with merged modules, when a merged library is being unmounted.
	 *
	 * @return The delegate.
	 */
	TTSMulticastDelegateRegistration<void()>& OnLibraryUnmount()
	{
		return OnLibraryUnmountCallback;
	}

	/**
	 * Gets a delegate that is executed whenever we need to confirm that modules can be safely unloaded.
	 *
	 * @return The delegate.
	 */
	TDelegateRegistration<bool(TConstArrayView<FName>)>& OnCheckLiveObjectsInModulesCallback()
	{
		return CheckLiveObjectsInModulesCallback;
	}

	/**
	 * Gets a delegate that is executed when a module containing UObjects has been loaded.
	 *
	 * The first parameter is the name of script package associated with the module.
	 *
	 * @return The event delegate.
	 */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsPackageLoadedCallback, FName);
	FIsPackageLoadedCallback& IsPackageLoadedCallback()
	{
		return IsPackageLoaded;
	}

protected:
	// FSelfRegisteringExec interface.
	CORE_API virtual bool Exec_Dev( UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar ) override;

private:
	friend TOptional<FModuleManager>& UE::Core::Private::GetModuleManagerSingleton();
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	/**
	 * Hidden constructor.
	 *
	 * Use the static Get function to return the singleton instance.
	 */
	CORE_API FModuleManager(FPrivateToken);
	FModuleManager(const FModuleManager&) = delete;
	FModuleManager& operator=(const FModuleManager&) = delete;

private:
	/**
	 * Information about a single module (may or may not be loaded.)
	 */
	class FModuleInfo
	{
	public:

		/** The original file name of the module, without any suffixes added */
		FString OriginalFilename;

		/** File name of this module (.dll file name) */
		FString Filename;

		/** Handle to this module (DLL handle), if it's currently loaded */
		void* Handle;

		/** The module object for this module.  We actually *own* this module, so it's lifetime is controlled by the scope of this shared pointer. */
		TUniquePtr<IModuleInterface> Module;

		/** True if this module was unloaded at shutdown time, and we never want it to be loaded again */
		bool bWasUnloadedAtShutdown;

		/** True if this module is full loaded and ready to be used */
		TAtomic<bool> bIsReady;

		/** Arbitrary number that encodes the load order of this module, so we can shut them down in reverse order. */
		int32 LoadOrder;

		/** static that tracks the current load number. Incremented whenever we add a new module*/
		static int32 CurrentLoadOrder;

	public:

		/** Constructor */
		FModuleInfo()
			: Handle(nullptr)
			, bWasUnloadedAtShutdown(false)
			, bIsReady(false)
			, LoadOrder(CurrentLoadOrder++)
		{ }

		~FModuleInfo() = default;
	};

	typedef TSharedPtr<FModuleInfo, ESPMode::ThreadSafe> ModuleInfoPtr;
	typedef TSharedRef<FModuleInfo, ESPMode::ThreadSafe> ModuleInfoRef;

	/** Type definition for maps of module names to module infos. */
	typedef TMap<FName, ModuleInfoRef> FModuleMap;

public:
	/**
	 * Generates a unique file name for the specified module name by adding a random suffix and checking for file collisions.
	 */
	CORE_API void MakeUniqueModuleFilename( const FName InModuleName, FString& UniqueSuffix, FString& UniqueModuleFileName ) const;

	CORE_API void AddModuleToModulesList(const FName InModuleName, FModuleManager::ModuleInfoRef& ModuleInfo);

	/** Clears module path cache */
	CORE_API void ResetModulePathsCache();

#if UE_MERGED_MODULES && !UE_BUILD_SHIPPING

	/** Dump all module usage for each loaded merged library */
	void LogMergedLibraryUsage();

	/** Load all modules from a merged library */
	void LoadAllModulesInMergedLibrary(FStringView LibraryName);

	/** Unload all modules from a merged library */
	void UnloadAllModulesInMergedLibrary(FStringView LibraryName);

#endif // UE_MERGED_MODULES && !UE_BUILD_SHIPPING

	/** Signal that objects used by an unloading dynamic library should be cleaned up now */
	CORE_API void OnObjectCleanup();

	friend FArchive& operator<<( FArchive& Ar, FModuleManager& ModuleManager );
private:
	static CORE_API void WarnIfItWasntSafeToLoadHere(const FName InModuleName);

	/** Thread safe module finding routine. */
	CORE_API ModuleInfoPtr FindModule(FName InModuleName);
	CORE_API ModuleInfoRef FindModuleChecked(FName InModuleName);

	UE_FORCEINLINE_HINT TSharedPtr<const FModuleInfo, ESPMode::ThreadSafe> FindModule(FName InModuleName) const
	{
		return const_cast<FModuleManager*>(this)->FindModule(InModuleName);
	}

	UE_FORCEINLINE_HINT TSharedRef<const FModuleInfo, ESPMode::ThreadSafe> FindModuleChecked(FName InModuleName) const
	{
		return const_cast<FModuleManager*>(this)->FindModuleChecked(InModuleName);
	}

#if !IS_MONOLITHIC
	/** Finds modules matching a given name wildcard. */
	CORE_API void FindModulePaths(const TCHAR *NamePattern, TMap<FName, FString> &OutModulePaths) const;

	/** Finds modules within a given directory. */
	CORE_API void FindModulePathsInDirectory(const FString &DirectoryName, bool bIsGameDirectory, TMap<FName, FString> &OutModulePaths) const;

	/** Serialize a bootstrapping state into or from an archive. */
	CORE_API void SerializeStateForBootstrap_Impl(FArchive& Ar);

	/** Refreshes the filename of a new module from the manifest */
	CORE_API void RefreshModuleFilenameFromManifestImpl(const FName InModuleName, FModuleInfo& ModuleInfo);

	/** Load the library for a given module */
	void* InternalLoadLibrary(FName ModuleName, const FString& ModuleFileToLoad);

	/** Attempt to free the backing library for a module */
	void InternalFreeLibrary(FName ModuleName, void* Handle);

#if UE_MERGED_MODULES

	/** Get all loaded modules linked into a particular merged library */
	TArray<FName> GetLoadedModulesForLibrary(const FString& LibraryFilename) const;

	/** Get all loaded merged libraries that depend on a particular shared merged library */
	TArray<FName> GetLoadedLibrariesForSharedLibrary(const FString& LibraryFilename) const;

	/** Get all shared library dependencies, loaded or not, for a particular merged library */
	TArray<FName> GetAllSharedLibrariesForLibrary(const FString& LibraryFilename) const;

#endif // UE_MERGED_MODULES

#endif // !IS_MONOLITHIC

	/** Adds pending module initializer registrations to the StaticallyLinkedModuleInitializers map. */
	CORE_API void ProcessPendingStaticallyLinkedModuleInitializers() const;

private:
	/** Map of all modules.  Maps the case-insensitive module name to information about that module, loaded or not. */
	FModuleMap Modules;

	/** Pending registrations of module names */
	/** We use an array here to stop comparisons (and thus FNames being constructed) when they are registered. */
	/** Instead, we validate there are no duplicates when they're inserted into StaticallyLinkedModuleInitializers. */
	mutable TArray<TPair<FLazyName, FInitializeStaticallyLinkedModule>, TInlineAllocator<16>> PendingStaticallyLinkedModuleInitializers;

	/** Map of module names to a delegate that can initialize each respective statically linked module */
	mutable TMap<FName, FInitializeStaticallyLinkedModule> StaticallyLinkedModuleInitializers;

	/** True if module manager should automatically register new UObjects discovered while loading C++ modules */
	bool bCanProcessNewlyLoadedObjects;

	/** True once AddExtraBinarySearchPaths has been called */
	bool bExtraBinarySearchPathsAdded;

	/** Check if we're in the middle of loading a dynamic library */
	bool bIsLoadingDynamicLibrary;

	/** Cache of known module paths. Used for performance. Can increase editor startup times by up to 30% */
	mutable TMap<FName, FString> ModulePathsCache;

	/** Multicast delegate that will broadcast a notification when modules are loaded, unloaded, or
		our set of known modules changes */
	TTSMulticastDelegate<void(FName ModuleName, EModuleChangeReason ChangeReason)> ModulesChangedEvent;
	
	/** Multicast delegate called to process any new loaded objects. */
	TMulticastDelegate<void(FName, bool)> ProcessLoadedObjectsCallback;

	/** Multicast delegate called to remove any loaded objects for a list of modules. */
	TMulticastDelegate<void(TConstArrayView<FName>)> OnModulesUnloadCallback;

	/** Multicast delegate called to remove anything left before a library unmounts. */
	TTSMulticastDelegate<void()> OnLibraryUnmountCallback;

	/** Delegate called to check whether any module to delay-unload still has live objects. */
	TDelegate<bool(TConstArrayView<FName>)> CheckLiveObjectsInModulesCallback;

	/** When module manager is linked against an application that supports UObjects, this delegate will be primed
		at startup to provide information about whether a UObject package is loaded into memory. */
	FIsPackageLoadedCallback IsPackageLoaded;

	/** Array of engine binaries directories. */
	mutable TArray<FString> EngineBinariesDirectories;
	mutable TArray<FString> PendingEngineBinariesDirectories;

	/** Array of game binaries directories. */
	mutable TArray<FString> GameBinariesDirectories;
	mutable TArray<FString> PendingGameBinariesDirectories;

	/** ID used to validate module manifests. Read from the module manifest in the engine directory on first query to load a new module; unset until then. */
	mutable TOptional<FString> BuildId;

	/** Critical section object controlling R/W access to Modules. */
	mutable FCriticalSection ModulesCriticalSection;

#if UE_MERGED_MODULES

	/** Path-to-handle map for libraries */
	TMap<FString, void*> LibraryHandles;

	/** Merged libraries that should be unloaded after the next object cleanup, with their associated modules */
	TMap<FString, TArray<FName>> DelayUnloadLibraries;

	/** Dependency map for merged libraries, for use with shared merged libraries */
	mutable TMap<FString, TArray<FString>> LibraryDependencies;

#endif // UE_MERGED_MODULES
};

/**
 * Utility class for registering modules that are statically linked.
 */
template< class ModuleClass >
class AUTORTFM_DISABLE FStaticallyLinkedModuleRegistrant
{
public:

	/**
	 * Explicit constructor that registers a statically linked module
	 */
	FStaticallyLinkedModuleRegistrant( FLazyName InModuleName )
	{
		FModuleManager::RegisterStaticallyLinkedModule(
			InModuleName,			// Module name
			&InitializeModule );	// Initializer
	}
	
	/**
	 * Creates and initializes this statically linked module.
	 *
	 * The module manager calls this function through the delegate that was created
	 * in the @see FStaticallyLinkedModuleRegistrant constructor.
	 *
	 * @return A pointer to a new instance of the module.
	 */
	static IModuleInterface* InitializeModule( )
	{
		return new ModuleClass();
	}
};

/**
 * Function pointer type for InitializeModule().
 *
 * All modules must have a FModuleInitializerEntry instance or an InitializeModule() function. Usually this is declared automatically using
 * the IMPLEMENT_MODULE macro below. If using the function it must be declared using as 'extern "C"' so that the
 * name remains undecorated. The object returned will be "owned" by the caller, and will be deleted
 * by the caller before the module is unloaded.
 */
typedef IModuleInterface* ( *FInitializeModuleFunctionPtr )( void );


/**
 * Intrusive linked list containing name and initializer function pointer of loaded modules.
 * Use this instead of "InitializeModule()" when possible
 */
class FModuleInitializerEntry
{
public:
	CORE_API FModuleInitializerEntry(const TCHAR* InName, FInitializeModuleFunctionPtr InFunction, const TCHAR* InName2 = nullptr);
	CORE_API ~FModuleInitializerEntry();

	static FInitializeModuleFunctionPtr FindModule(const TCHAR* Name);

private:
	FModuleInitializerEntry* Prev;
	FModuleInitializerEntry* Next;
	const TCHAR* Name;
	const TCHAR* Name2;
	FInitializeModuleFunctionPtr Function;
};


/**
 * A default minimal implementation of a module that does nothing at startup and shutdown
 */
class FDefaultModuleImpl
	: public IModuleInterface
{ };


/**
 * Default minimal module class for gameplay modules.  Does nothing at startup and shutdown.
 */
class FDefaultGameModuleImpl
	: public FDefaultModuleImpl
{
	/**
	 * Returns true if this module hosts gameplay code
	 *
	 * @return True for "gameplay modules", or false for engine code modules, plug-ins, etc.
	 */
	virtual bool IsGameModule() const override
	{
		return true;
	}
};

/**
 * Module implementation boilerplate for regular modules.
 *
 * This macro is used to expose a module's main class to the rest of the engine.
 * You must use this macro in one of your modules C++ modules, in order for the 'InitializeModule'
 * function to be declared in such a way that the engine can find it. Also, this macro will handle
 * the case where a module is statically linked with the engine instead of dynamically loaded.
 *
 * This macro is intended for modules that do NOT contain gameplay code.
 * If your module does contain game classes, use IMPLEMENT_GAME_MODULE instead.
 *
 * Usage:   IMPLEMENT_MODULE(<My Module Class>, <Module name string>)
 *
 * @see IMPLEMENT_GAME_MODULE
 */

namespace UE::Core::Private
{
	constexpr bool ModuleNameEquals(const char* Lhs, const char* Rhs)
	{
		for (;;)
		{
			if (*Lhs != *Rhs)
			{
				return false;
			}

			if (*Lhs == '\0')
			{
				return true;
			}

			++Lhs;
			++Rhs;
		}
	}
}
#if IS_MONOLITHIC || UE_MERGED_MODULES

	// If we're linking monolithically we assume all modules are linked in with the main binary.
	#define IMPLEMENT_MODULE( ModuleImplClass, ModuleName ) \
		/** Global registrant object for this module when linked statically */ \
		static FStaticallyLinkedModuleRegistrant< ModuleImplClass > ModuleRegistrant##ModuleName( TEXT(#ModuleName) ); \
		/* Forced reference to this function is added by the linker to check that each module uses IMPLEMENT_MODULE */ \
		extern "C" AUTORTFM_DISABLE void IMPLEMENT_MODULE_##ModuleName() { UE_STATIC_ASSERT_WARN(UE::Core::Private::ModuleNameEquals(#ModuleName, UE_MODULE_NAME ), "Module name mismatch (" #ModuleName " != " UE_MODULE_NAME "). Please ensure module name passed to IMPLEMENT_MODULE is " UE_MODULE_NAME " to avoid runtime errors in monolithic builds."); } \
		PER_MODULE_BOILERPLATE_ANYLINK(ModuleImplClass, ModuleName)

#else

	#define IMPLEMENT_MODULE( ModuleImplClass, ModuleName ) \
		\
		/**/ \
		/* InitializeModule function, called by module manager after this module's DLL has been loaded */ \
		/**/ \
		/* @return	Returns an instance of this module */ \
		/**/ \
		AUTORTFM_DISABLE static IModuleInterface* Initialize##ModuleName##Module() \
		{ \
			return new ModuleImplClass(); \
		} \
		static FModuleInitializerEntry ModuleName##InitializerEntry(TEXT(#ModuleName), Initialize##ModuleName##Module, TEXT(UE_MODULE_NAME)); \
		/* Forced reference to this function is added by the linker to check that each module uses IMPLEMENT_MODULE */ \
		extern "C" AUTORTFM_DISABLE void IMPLEMENT_MODULE_##ModuleName() { UE_STATIC_ASSERT_WARN(UE::Core::Private::ModuleNameEquals(#ModuleName, UE_MODULE_NAME ), "Module name mismatch (" #ModuleName " != " UE_MODULE_NAME "). Please ensure module name passed to IMPLEMENT_MODULE is " UE_MODULE_NAME " to avoid runtime errors in monolithic builds."); } \
		PER_MODULE_BOILERPLATE_ANYLINK(ModuleImplClass, ModuleName)

#endif //IS_MONOLITHIC


/**
 * Module implementation boilerplate for game play code modules.
 *
 * This macro works like IMPLEMENT_MODULE but is specifically used for modules that contain game play code.
 * If your module does not contain game classes, use IMPLEMENT_MODULE instead.
 *
 * Usage:   IMPLEMENT_GAME_MODULE(<My Game Module Class>, <Game Module name string>)
 *
 * @see IMPLEMENT_MODULE
 */
#define IMPLEMENT_GAME_MODULE( ModuleImplClass, ModuleName ) \
	IMPLEMENT_MODULE( ModuleImplClass, ModuleName )


/**
 * Macro for declaring the engine directory to check for foreign or nested projects.
 */
#if PLATFORM_DESKTOP
	#ifdef UE_ENGINE_DIRECTORY
		#define IMPLEMENT_FOREIGN_ENGINE_DIR() const TCHAR *GForeignEngineDir = TEXT( UE_ENGINE_DIRECTORY );
	#else
		#define IMPLEMENT_FOREIGN_ENGINE_DIR() const TCHAR *GForeignEngineDir = nullptr;
	#endif
#else
	#define IMPLEMENT_FOREIGN_ENGINE_DIR() 
#endif

/**
 * Macro for passing a list argument to a macro
 */
#define UE_LIST_ARGUMENT(...) __VA_ARGS__

/**
 * Macro for registering signing keys for a project.
 */
#define UE_REGISTER_SIGNING_KEY(ExponentValue, ModulusValue) \
	struct FSigningKeyRegistration \
	{ \
		FSigningKeyRegistration() \
		{ \
			extern CORE_API void RegisterSigningKeyCallback(void (*)(TArray<uint8>&, TArray<uint8>&)); \
			RegisterSigningKeyCallback(&Callback); \
		} \
		static void Callback(TArray<uint8>& OutExponent, TArray<uint8>& OutModulus) \
		{ \
			const uint8 Exponent[] = { ExponentValue }; \
			const uint8 Modulus[] = { ModulusValue }; \
			OutExponent.SetNum(UE_ARRAY_COUNT(Exponent)); \
			OutModulus.SetNum(UE_ARRAY_COUNT(Modulus)); \
			for(int ByteIdx = 0; ByteIdx < UE_ARRAY_COUNT(Exponent); ByteIdx++) \
			{ \
				OutExponent[ByteIdx] = Exponent[ByteIdx]; \
			} \
			for(int ByteIdx = 0; ByteIdx < UE_ARRAY_COUNT(Modulus); ByteIdx++) \
			{ \
				OutModulus[ByteIdx] = Modulus[ByteIdx]; \
			} \
		} \
	} GSigningKeyRegistration;

/**
 * Macro for registering encryption key for a project.
 */
#define UE_REGISTER_ENCRYPTION_KEY(...) \
	struct FEncryptionKeyRegistration \
	{ \
		FEncryptionKeyRegistration() \
		{ \
			extern CORE_API void RegisterEncryptionKeyCallback(void (*)(unsigned char OutKey[32])); \
			RegisterEncryptionKeyCallback(&Callback); \
		} \
		static void Callback(unsigned char OutKey[32]) \
		{ \
			const unsigned char Key[32] = { __VA_ARGS__ }; \
			for(int ByteIdx = 0; ByteIdx < 32; ByteIdx++) \
			{ \
				OutKey[ByteIdx] = Key[ByteIdx]; \
			} \
		} \
	} GEncryptionKeyRegistration;

#define IMPLEMENT_TARGET_NAME_REGISTRATION() \
	struct FTargetNameRegistration \
	{ \
		FTargetNameRegistration() \
		{ \
			FPlatformMisc::SetUBTTargetName(TEXT(UE_STRINGIZE(UE_TARGET_NAME))); \
		} \
	} GTargetNameRegistration;

#if IS_PROGRAM

	#if IS_MONOLITHIC
		#define IMPLEMENT_APPLICATION( ModuleName, GameName ) \
			/* For monolithic builds, we must statically define the game's name string (See Core.h) */ \
			TCHAR GInternalProjectName[64] = TEXT( GameName ); \
			IMPLEMENT_FOREIGN_ENGINE_DIR() \
			IMPLEMENT_SIGNING_KEY_REGISTRATION() \
			IMPLEMENT_ENCRYPTION_KEY_REGISTRATION() \
			IMPLEMENT_GAME_MODULE(FDefaultGameModuleImpl, ModuleName) \
			PER_MODULE_BOILERPLATE \
			FEngineLoop GEngineLoop;

	#else		

		#define IMPLEMENT_APPLICATION( ModuleName, GameName ) \
			/* For non-monolithic programs, we must set the game's name string before main starts (See Core.h) */ \
			struct FAutoSet##ModuleName \
			{ \
				FAutoSet##ModuleName() \
				{ \
					FCString::Strncpy(GInternalProjectName, TEXT( GameName ), UE_ARRAY_COUNT(GInternalProjectName)); \
				} \
			} AutoSet##ModuleName; \
			PER_MODULE_BOILERPLATE_ANYLINK(FDefaultGameModuleImpl, ModuleName) \
			FEngineLoop GEngineLoop;
	#endif

#else

/** IMPLEMENT_PRIMARY_GAME_MODULE must be used for at least one game module in your game.  It sets the "name"
	your game when compiling in monolithic mode. This is passed in by UBT from the .uproject name, and manually specifying a 
	name is no longer necessary. */
#if IS_MONOLITHIC
	#if PLATFORM_DESKTOP

		#define IMPLEMENT_PRIMARY_GAME_MODULE( ModuleImplClass, ModuleName, DEPRECATED_GameName ) \
			/* For monolithic builds, we must statically define the game's name string (See Core.h) */ \
			TCHAR GInternalProjectName[64] = TEXT( UE_STRINGIZE(UE_PROJECT_NAME) ); \
			/* Implement the GIsGameAgnosticExe variable (See Core.h). */ \
			bool GIsGameAgnosticExe = false; \
			IMPLEMENT_FOREIGN_ENGINE_DIR() \
			IMPLEMENT_SIGNING_KEY_REGISTRATION() \
			IMPLEMENT_ENCRYPTION_KEY_REGISTRATION() \
			IMPLEMENT_TARGET_NAME_REGISTRATION() \
			IMPLEMENT_GAME_MODULE( ModuleImplClass, ModuleName ) \
			PER_MODULE_BOILERPLATE

	#else	//PLATFORM_DESKTOP

		#define IMPLEMENT_PRIMARY_GAME_MODULE( ModuleImplClass, ModuleName, DEPRECATED_GameName ) \
			/* For monolithic builds, we must statically define the game's name string (See Core.h) */ \
			TCHAR GInternalProjectName[64] = TEXT( UE_STRINGIZE(UE_PROJECT_NAME) ); \
			PER_MODULE_BOILERPLATE \
			IMPLEMENT_FOREIGN_ENGINE_DIR() \
			IMPLEMENT_SIGNING_KEY_REGISTRATION() \
			IMPLEMENT_ENCRYPTION_KEY_REGISTRATION() \
			IMPLEMENT_TARGET_NAME_REGISTRATION() \
			IMPLEMENT_GAME_MODULE( ModuleImplClass, ModuleName ) \
			/* Implement the GIsGameAgnosticExe variable (See Core.h). */ \
			bool GIsGameAgnosticExe = false;

	#endif	//PLATFORM_DESKTOP

#else	//IS_MONOLITHIC

	#define IMPLEMENT_PRIMARY_GAME_MODULE( ModuleImplClass, ModuleName, GameName ) \
		/* Nothing special to do for modular builds.  The game name will be set via the command-line */ \
		IMPLEMENT_SIGNING_KEY_REGISTRATION() \
		IMPLEMENT_ENCRYPTION_KEY_REGISTRATION() \
		IMPLEMENT_TARGET_NAME_REGISTRATION() \
		IMPLEMENT_GAME_MODULE( ModuleImplClass, ModuleName )
#endif	//IS_MONOLITHIC

#endif

/**
* Enumerates the type of reload in progress
*/
enum class EActiveReloadType
{
	None,
	Reinstancing,
#if WITH_HOT_RELOAD
	HotReload,
#endif
#if WITH_LIVE_CODING
	LiveCoding,
#endif
};

class IReload;

#if WITH_RELOAD
/**
* Return the currently active reload.  Check for None to see if reloading is not active.
* This method respects the GIsHotReload setting.
*/
CORE_API EActiveReloadType GetActiveReloadType();
/**
* Get the currently active reload interface.
*/
CORE_API IReload* GetActiveReloadInterface();

/**
* Helper method to check to see if reloading is active.
* This method respects the GIsHotReload setting.
*/
CORE_API bool IsReloadActive();

/**
* Begins the reload process.
*/
CORE_API void BeginReload(EActiveReloadType ActiveReloadType, IReload& Interface);

/**
* Ends the reload process
*/
CORE_API void EndReload();
#else
inline EActiveReloadType GetActiveReloadType() { return EActiveReloadType::None; }

inline bool IsReloadActive() { return false; }

inline IReload* GetActiveReloadInterface() { return nullptr; }
#endif
