// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Modules/BuildVersion.h"

/**
 * Stores a record of a built target, with all metadata that other tools may need to know about the build.
 */
class FModuleManifest
{
public:
	FString BuildId;
	TMap<FString, FString> ModuleNameToFileName;
	TMap<FString, TArray<FString>> LibraryDependencies;

	/**
	 * Default constructor 
	 */
	CORE_API FModuleManifest();

	/**
	 * Gets the path to a version manifest for the given folder.
	 *
	 * @param DirectoryName		Directory to read from
	 * @param bIsGameFolder		Whether the directory is a game folder of not. Used to adjust the name if the application is running in DebugGame.
	 * @return The filename of the version manifest.
	 */
	static CORE_API FString GetFileName(const FString& DirectoryName, bool bIsGameFolder);

	/**
	 * Read a version manifest from disk.
	 *
	 * @param FileName		Filename to read from
	 */
	static CORE_API bool TryRead(const FString& FileName, FModuleManifest& Manifest);
};
