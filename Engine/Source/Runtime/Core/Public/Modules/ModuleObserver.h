// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

/** A simple class that observes the currently active module list for a specific module to be (un)loaded */
class FModuleObserver
{
public:

	/**
	 * Constructor
	 * @param InModuleName			The name of the module we want to observe
	 * @param InOnModuleLoaded		Called when the module is loaded (or immediately, if it is already loaded)
	 * @param InOnModuleUnloaded	Called when the module is unloaded (or on destruction, if it is still loaded)
	 */
	FModuleObserver(FName InModuleName, FSimpleDelegate InOnModuleLoaded, FSimpleDelegate InOnModuleUnloaded)
		: ModuleName(InModuleName)
		, OnModuleLoaded(InOnModuleLoaded)
		, OnModuleUnloaded(InOnModuleUnloaded)
	{
		FModuleManager& ModuleManager = FModuleManager::Get();

		if (ModuleManager.IsModuleLoaded(ModuleName))
		{
			OnModuleLoaded.ExecuteIfBound();
		}

		ModuleManager.OnModulesChanged().AddRaw(this, &FModuleObserver::OnModulesChanged);
	}

	~FModuleObserver()
	{
		FModuleManager& ModuleManager = FModuleManager::Get();
		
		if (ModuleManager.IsModuleLoaded(ModuleName))
		{
			OnModuleUnloaded.ExecuteIfBound();
		}

		ModuleManager.OnModulesChanged().RemoveAll(this);
	}

private:

	/** Called whenever the module list changes at all */
	void OnModulesChanged(FName InModuleName, EModuleChangeReason Reason)
	{
		if (InModuleName != ModuleName || Reason == EModuleChangeReason::PluginDirectoryChanged)
		{
			return;
		}

		switch (Reason)
		{
			case EModuleChangeReason::ModuleLoaded:		OnModuleLoaded.ExecuteIfBound();		break;
			case EModuleChangeReason::ModuleUnloaded:	OnModuleUnloaded.ExecuteIfBound();		break;
		}
	}

private:
	/** The name of the module we are observing */
	FName ModuleName;
	/** Called when the module is loaded */
	FSimpleDelegate OnModuleLoaded;
	/** Called when the module is unloaded */
	FSimpleDelegate OnModuleUnloaded;
};
