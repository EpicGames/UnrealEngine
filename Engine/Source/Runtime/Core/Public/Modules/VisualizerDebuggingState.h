// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#ifndef UE_VISUALIZER_DEBUGGING_STATE
	#define UE_VISUALIZER_DEBUGGING_STATE !UE_BUILD_SHIPPING
#endif

struct FGuid;

namespace UE::Core
{

struct FVisualizerDebuggingStateImpl;

/** Enumeration specifying how a call to FVisualizerDebuggingState::Assign succeeded or failed */
enum class EVisualizerDebuggingStateResult : uint8
{
	/** The operation completed successfully */
	Success,
	/** Failure: The supplied guid happened to collide with the concatenation of 2 other pre-assigned GUIDs Please use a different GUID. */
	StringCollision,
	/** Failure: The feature is disabled entirely. */
	FeatureDisabled,
};

/**
 * Global visualizer state manager that handles the complexities of natvis lookup rules for DLLs and LiveCoding
 *
 * The singleton instance of this class is forward declared in CoreGlobals.h (GCoreDebuggingState), with
 *   additional symbols added to each DLL through UE_VISUALIZERS_HELPERS (GDebuggingState).
 *
 * When a plugin or engine system needs to provide a globally accessible pointer to a natvis visualizer,
 *   that pointer can be added by calling FVisualizerDebuggingState::Assign using a globally unique ID
 *   that is used to find the pointer in the natvis. GUIDs may be generated using any readily-available GUID generator tool.

 * Pointers are located using an intrinsic call to strstr that finds the pointer's offset using a string match
 *   of the original GUID's lower-case, 32-character string representation of the form "c198d347ec1243ec833c423bd5fb6084".
 *   The offset of the address of the  match divided by 32 gives the entry index from which the actual debug ptr can be retrieved.
 *
 * An example visualizer syntax is included below. It is recommended that most natvis files utilizing this method would include an
 *   global intrinsic function at the top for quick access:
 * 
 *   <Intrinsic Name="GetDebugState" Expression="GDebuggingState->Ptrs[(strstr(GDebuggingState->GuidString, GuidString) - GDebuggingState->GuidString)/32]">
 *     <Parameter Name="GuidString" Type="char*"/>
 *   </Intrinsic>
 *
 *   // C++:
 *   struct FMyGlobalState
 *   {
 *     static FMyGlobalState* GetSingleton();
 *     TArray<FString> TypeNames;
 *   };
 *   struct FMyStruct
 *   {
 *     int32 TypeNameIndex;
 *   };
 * 
 *   // Register the global state
 *   FGuid DebugVisualizerID(0x1bdc1747, 0x6b924697, 0xbd8fccc8, 0xb26b4f79)
 *   EVisualizerDebuggingStateResult Result = FVisualizerDebuggingState::Assign(DebugVisualizerID, FMyGlobalState::GetSingleton());
 *
 *   // .natvis:
 *   <!-- Example natvis syntax for finding a FMyGlobalState* registered with the guid 1bdc1747-6b92-4697-bd8f-ccc8b26b4f79 -->
 *   <Type Name="FMyStruct">
 *     <!-- less verbose, more fragile version -->
 *     <Intrinsic Name="GetGlobalState" Expression="((FMyGlobalState*)GetDebugState(&quot;1bdc17476b924697bd8fccc8b26b4f79&quot;))""></Intrinsic>
 *
 *     <DisplayString>Type={ GetGlobalState()->TypeNames[TypeNameIndex] }</DisplayString>
 *   </Type>
 */
struct FVisualizerDebuggingState
{
	FVisualizerDebuggingState();

	FVisualizerDebuggingState(const FVisualizerDebuggingState&) = delete;
	void operator=(const FVisualizerDebuggingState&) = delete;

	FVisualizerDebuggingState(FVisualizerDebuggingState&&) = delete;
	void operator=(FVisualizerDebuggingState&&) = delete;

	~FVisualizerDebuggingState();

#if !UE_VISUALIZER_DEBUGGING_STATE

	/**
	 * Assign a globally accessible debugging state ptr by name, potentially overwriting a previously assigned ptr
	 * 
	 * @param UniqueId   A guid that uniquely identifies this debug ptr. This should be the same as is used to access the ptr inside a natvis file.
	 * @param DebugPtr   The pointer that needs to be accessible from natvis
	 * @return EVisualizerDebuggingStateResult::Success if the operation completed successfully, otherwise an error code
	 */
	[[nodiscard]] static EVisualizerDebuggingStateResult Assign(const FGuid& UniqueId, void* DebugPtr)
	{
		return EVisualizerDebuggingStateResult::FeatureDisabled;
	}

#else

	/**
	 * Assign a globally accessible debugging state ptr by name, potentially overwriting a previously assigned ptr
	 * 
	 * @param UniqueId   A guid that uniquely identifies this debug ptr. This should be the same as is used to access the ptr inside a natvis file.
	 * @param DebugPtr   The pointer that needs to be accessible from natvis
	 * @return EVisualizerDebuggingStateResult::Success if the operation completed successfully, otherwise an error code
	 */
	[[nodiscard]] CORE_API static EVisualizerDebuggingStateResult Assign(const FGuid& UniqueId, void* DebugPtr);


protected:
	// protected in order to support automation test introspection

	/**
	 * Implementation function for Assign
	 */
	EVisualizerDebuggingStateResult AssignImpl(const FGuid& UniqueId, void* DebugPtr);

	/**
	 * Attempt to locate the index of the specified unique ID
	 */
	void* Find(const FGuid& UniqueId) const;

	/**
	 * Single string containing all GUID strings ordered by their insertion order.
	 * The single string is designed to support finding a string offset using a single strstr instrinsic call, and dividing the result by 32.
	 * This string exists for easy referencing inside natvis files; it points to PimplData->OrderedGuidString.
	 * Example layout:
	 *   
	 *   OrderedGuidString: "d9ad42709d2c4bc8a2f8f925e1617b288456cfc3222f4833a8afa45a6ed73b5a9873a9fedd8441d49ef6b3258e8a6c60#"
	 *   Ptrs:              [Ptr1                           ,Ptr2                           ,Ptr3]
	 */
	const char* GuidString = nullptr;

	/**
	 * Array of void* ptrs to the user-provided debugging state ptrs for each entry.
	 * Points directly to PimplData->DebugPtrs.GetData() for quick referencing inside natvis without needing any other symbols or casting.
	 */
	void** Ptrs = nullptr;

	/** Pimpl to avoid pulling in too many headers since this header is included everywhere */
	FVisualizerDebuggingStateImpl* PimplData = nullptr;

#endif
};

} // namespace UE::Core
