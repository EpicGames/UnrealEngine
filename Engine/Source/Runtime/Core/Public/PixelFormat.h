// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/CoreReflectedTypeAccessors.h"

#if defined(PF_MAX)
#undef PF_MAX
#endif

enum class EPixelFormatCapabilities : uint32;

enum EPixelFormat : uint8
{
	PF_Unknown              =0,
	PF_A32B32G32R32F        =1,
	PF_B8G8R8A8             =2,
	PF_G8                   =3, // G8  means Gray/Grey , not Green , typically actually uses a red format with replication of R to RGB
	PF_G16                  =4, // G16 means Gray/Grey like G8
	PF_DXT1                 =5,
	PF_DXT3                 =6,
	PF_DXT5                 =7,
	PF_UYVY                 =8,
	PF_FloatRGB             =9,  // FloatRGB == PF_FloatR11G11B10 , NOT 16F usually, but varies
	PF_FloatRGBA            =10, // RGBA16F
	PF_DepthStencil         =11,
	PF_ShadowDepth          =12,
	PF_R32_FLOAT            =13,
	PF_G16R16               =14,
	PF_G16R16F              =15,
	PF_G16R16F_FILTER       =16,
	PF_G32R32F              =17,
	PF_A2B10G10R10          =18,
	PF_A16B16G16R16         =19,
	PF_D24                  =20,
	PF_R16F                 =21,
	PF_R16F_FILTER          =22,
	PF_BC5                  =23,
	PF_V8U8                 =24,
	PF_A1                   =25,
	PF_FloatR11G11B10       =26,
	PF_A8                   =27,
	PF_R32_UINT             =28,
	PF_R32_SINT             =29,
	PF_PVRTC2               =30,
	PF_PVRTC4               =31,
	PF_R16_UINT             =32,
	PF_R16_SINT             =33,
	PF_R16G16B16A16_UINT    =34,
	PF_R16G16B16A16_SINT    =35,
	PF_R5G6B5_UNORM         =36,
	PF_R8G8B8A8             =37,
	PF_A8R8G8B8				=38,	// Only used for legacy loading; do NOT use!
	PF_BC4					=39,
	PF_R8G8                 =40,	
	PF_ATC_RGB				=41,	// Unsupported Format
	PF_ATC_RGBA_E			=42,	// Unsupported Format
	PF_ATC_RGBA_I			=43,	// Unsupported Format
	PF_X24_G8				=44,	// Used for creating SRVs to alias a DepthStencil buffer to read Stencil. Don't use for creating textures.
	PF_ETC1					=45,	// Unsupported Format
	PF_ETC2_RGB				=46,
	PF_ETC2_RGBA			=47,
	PF_R32G32B32A32_UINT	=48,
	PF_R16G16_UINT			=49,
	PF_ASTC_4x4             =50,	// 8.00 bpp
	PF_ASTC_6x6             =51,	// 3.56 bpp
	PF_ASTC_8x8             =52,	// 2.00 bpp
	PF_ASTC_10x10           =53,	// 1.28 bpp
	PF_ASTC_12x12           =54,	// 0.89 bpp
	PF_BC6H					=55,
	PF_BC7					=56,
	PF_R8_UINT				=57,
	PF_L8					=58,
	PF_XGXR8				=59,
	PF_R8G8B8A8_UINT		=60,
	PF_R8G8B8A8_SNORM		=61,
	PF_R16G16B16A16_UNORM	=62,
	PF_R16G16B16A16_SNORM	=63,
	PF_PLATFORM_HDR_0		=64,
	PF_PLATFORM_HDR_1		=65,	// Reserved.
	PF_PLATFORM_HDR_2		=66,	// Reserved.
	PF_NV12					=67,
	PF_R32G32_UINT          =68,
	PF_ETC2_R11_EAC			=69,
	PF_ETC2_RG11_EAC		=70,
	PF_R8		            =71,
	PF_B5G5R5A1_UNORM       =72,
	PF_ASTC_4x4_HDR         =73,	
	PF_ASTC_6x6_HDR         =74,	
	PF_ASTC_8x8_HDR         =75,	
	PF_ASTC_10x10_HDR       =76,	
	PF_ASTC_12x12_HDR       =77,
	PF_G16R16_SNORM			=78,
	PF_R8G8_UINT			=79,
	PF_R32G32B32_UINT		=80,
	PF_R32G32B32_SINT		=81,
	PF_R32G32B32F			=82,
	PF_R8_SINT				=83,	
	PF_R64_UINT				=84,
	PF_R9G9B9EXP5			=85,
	PF_P010					=86,
	PF_ASTC_4x4_NORM_RG		=87, // RG format stored in LA endpoints for better precision (requires RHI support for texture swizzle)
	PF_ASTC_6x6_NORM_RG		=88,	
	PF_ASTC_8x8_NORM_RG		=89,	
	PF_ASTC_10x10_NORM_RG	=90,	
	PF_ASTC_12x12_NORM_RG	=91,	
	PF_R16G16_SINT			=92,
	PF_R8G8B8				=93,
	PF_MAX					=94,

	// If updating this enum, remember to update FOREACH_ENUM_EPIXELFORMAT_CORE below.
};

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<EPixelFormat>();
#endif

// This should match the definition of FOREACH_ENUM_EPIXELFORMAT generated in NoExportTypes.generated.h.
// It is defined separately here to prevent clashes of the macro name when NoExportTypes.h is also included.
#define FOREACH_ENUM_EPIXELFORMAT_CORE(op) \
	op(PF_Unknown) \
	op(PF_A32B32G32R32F) \
	op(PF_B8G8R8A8) \
	op(PF_G8) \
	op(PF_G16) \
	op(PF_DXT1) \
	op(PF_DXT3) \
	op(PF_DXT5) \
	op(PF_UYVY) \
	op(PF_FloatRGB) \
	op(PF_FloatRGBA) \
	op(PF_DepthStencil) \
	op(PF_ShadowDepth) \
	op(PF_R32_FLOAT) \
	op(PF_G16R16) \
	op(PF_G16R16F) \
	op(PF_G16R16F_FILTER) \
	op(PF_G32R32F) \
	op(PF_A2B10G10R10) \
	op(PF_A16B16G16R16) \
	op(PF_D24) \
	op(PF_R16F) \
	op(PF_R16F_FILTER) \
	op(PF_BC5) \
	op(PF_V8U8) \
	op(PF_A1) \
	op(PF_FloatR11G11B10) \
	op(PF_A8) \
	op(PF_R32_UINT) \
	op(PF_R32_SINT) \
	op(PF_PVRTC2) \
	op(PF_PVRTC4) \
	op(PF_R16_UINT) \
	op(PF_R16_SINT) \
	op(PF_R16G16B16A16_UINT) \
	op(PF_R16G16B16A16_SINT) \
	op(PF_R5G6B5_UNORM) \
	op(PF_R8G8B8A8) \
	op(PF_A8R8G8B8) \
	op(PF_BC4) \
	op(PF_R8G8) \
	op(PF_ATC_RGB) \
	op(PF_ATC_RGBA_E) \
	op(PF_ATC_RGBA_I) \
	op(PF_X24_G8) \
	op(PF_ETC1) \
	op(PF_ETC2_RGB) \
	op(PF_ETC2_RGBA) \
	op(PF_R32G32B32A32_UINT) \
	op(PF_R16G16_UINT) \
	op(PF_ASTC_4x4) \
	op(PF_ASTC_6x6) \
	op(PF_ASTC_8x8) \
	op(PF_ASTC_10x10) \
	op(PF_ASTC_12x12) \
	op(PF_BC6H) \
	op(PF_BC7) \
	op(PF_R8_UINT) \
	op(PF_L8) \
	op(PF_XGXR8) \
	op(PF_R8G8B8A8_UINT) \
	op(PF_R8G8B8A8_SNORM) \
	op(PF_R16G16B16A16_UNORM) \
	op(PF_R16G16B16A16_SNORM) \
	op(PF_PLATFORM_HDR_0) \
	op(PF_PLATFORM_HDR_1) \
	op(PF_PLATFORM_HDR_2) \
	op(PF_NV12) \
	op(PF_R32G32_UINT) \
	op(PF_ETC2_R11_EAC) \
	op(PF_ETC2_RG11_EAC) \
	op(PF_R8) \
	op(PF_B5G5R5A1_UNORM) \
	op(PF_ASTC_4x4_HDR) \
	op(PF_ASTC_6x6_HDR) \
	op(PF_ASTC_8x8_HDR) \
	op(PF_ASTC_10x10_HDR) \
	op(PF_ASTC_12x12_HDR) \
	op(PF_G16R16_SNORM) \
	op(PF_R8G8_UINT) \
	op(PF_R32G32B32_UINT) \
	op(PF_R32G32B32_SINT) \
	op(PF_R32G32B32F) \
	op(PF_R8_SINT) \
	op(PF_R64_UINT) \
	op(PF_R9G9B9EXP5) \
	op(PF_P010) \
	op(PF_ASTC_4x4_NORM_RG) \
	op(PF_ASTC_6x6_NORM_RG) \
	op(PF_ASTC_8x8_NORM_RG) \
	op(PF_ASTC_10x10_NORM_RG) \
	op(PF_ASTC_12x12_NORM_RG) \
	op(PF_R16G16_SINT) \
	op(PF_R8G8B8)

// Defines which channel is valid for each pixel format
enum class EPixelFormatChannelFlags : uint8
{
	R = 1 << 0,
	G = 1 << 1,
	B = 1 << 2,
	A = 1 << 3,
	RG = R | G,
	RGB = R | G | B,
	RGBA = R | G | B | A,

	None = 0,
};
ENUM_CLASS_FLAGS(EPixelFormatChannelFlags);

enum class EPixelFormatCapabilities : uint32
{
    None             = 0,
    Texture1D        = 1ull << 1,
    Texture2D        = 1ull << 2,
    Texture3D        = 1ull << 3,
    TextureCube      = 1ull << 4,
    RenderTarget     = 1ull << 5,
    DepthStencil     = 1ull << 6,
	TextureMipmaps   = 1ull << 7,
	TextureLoad      = 1ull << 8,
	TextureSample    = 1ull << 9,
	TextureGather    = 1ull << 10,
	TextureAtomics   = 1ull << 11,
	TextureBlendable = 1ull << 12,
	TextureStore     = 1ull << 13,

	Buffer           = 1ull << 14,
    VertexBuffer     = 1ull << 15,
    IndexBuffer      = 1ull << 16,
	BufferLoad       = 1ull << 17,
    BufferStore      = 1ull << 18,
    BufferAtomics    = 1ull << 19,

	UAV              = 1ull << 20,
    TypedUAVLoad     = 1ull << 21,
	TypedUAVStore    = 1ull << 22,

	TextureFilterable = 1ull << 23,
	LossyCompressible = 1ull << 24,

	AnyTexture       = Texture1D | Texture2D | Texture3D | TextureCube,

	AllTextureFlags  = AnyTexture | RenderTarget | DepthStencil | TextureMipmaps | TextureLoad | TextureSample | TextureGather | TextureAtomics | TextureBlendable | TextureStore,
	AllBufferFlags   = Buffer | VertexBuffer | IndexBuffer | BufferLoad | BufferStore | BufferAtomics,
	AllUAVFlags      = UAV | TypedUAVLoad | TypedUAVStore,

	AllFlags         = AllTextureFlags | AllBufferFlags | AllUAVFlags
};
ENUM_CLASS_FLAGS(EPixelFormatCapabilities);

// EPixelFormat is currently used interchangably with uint8, and most call sites taking a uint8
// should be updated to take an EPixelFormat instead, but in the interim this allows fixing
// type conversion warnings
#define UE_PIXELFORMAT_TO_UINT8(argument) static_cast<uint8>(argument)

inline bool IsETCBlockCompressedPixelFormat(EPixelFormat PixelFormat)
{
	PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	switch (PixelFormat)
	{
	case PF_ETC2_RGB:
	case PF_ETC2_RGBA:
	case PF_ETC2_R11_EAC:
	case PF_ETC2_RG11_EAC:
		return true;
	}
	PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	return false;
}

inline bool IsASTCBlockCompressedTextureFormat(EPixelFormat PixelFormat)
{
	PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	switch (PixelFormat)
	{
	case PF_ASTC_4x4:
	case PF_ASTC_6x6:
	case PF_ASTC_8x8:
	case PF_ASTC_10x10:
	case PF_ASTC_12x12:
	case PF_ASTC_4x4_HDR:
	case PF_ASTC_6x6_HDR:
	case PF_ASTC_8x8_HDR:
	case PF_ASTC_10x10_HDR:
	case PF_ASTC_12x12_HDR:
	case PF_ASTC_4x4_NORM_RG:
	case PF_ASTC_6x6_NORM_RG:
	case PF_ASTC_8x8_NORM_RG:
	case PF_ASTC_10x10_NORM_RG:
	case PF_ASTC_12x12_NORM_RG:
		return true;
	default:
		return false;
	}
	PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
}

inline bool IsBlockCompressedFormat(EPixelFormat Format)
{
	PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	switch (Format)
	{
	case PF_DXT1:
	case PF_DXT3:
	case PF_DXT5:
	case PF_BC4:
	case PF_BC5:
	case PF_BC6H:
	case PF_BC7:
		return true;
	}
	PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	return false;
}

UE_FORCEINLINE_HINT bool IsAnyBlockCompressedPixelFormat(EPixelFormat PixelFormat)
{
	return IsBlockCompressedFormat(PixelFormat) || IsASTCBlockCompressedTextureFormat(PixelFormat) || IsETCBlockCompressedPixelFormat(PixelFormat);
}

// unclear what IsHDR is supposed to mean
//	see also IsFloatFormat
inline bool IsHDR(EPixelFormat PixelFormat)
{
	return PixelFormat == PF_FloatRGBA || PixelFormat == PF_BC6H || PixelFormat == PF_R16F || PixelFormat == PF_R32_FLOAT || PixelFormat == PF_A32B32G32R32F
		|| PixelFormat == PF_ASTC_4x4_HDR || PixelFormat == PF_ASTC_6x6_HDR || PixelFormat == PF_ASTC_8x8_HDR || PixelFormat == PF_ASTC_10x10_HDR || PixelFormat == PF_ASTC_12x12_HDR;
}

inline bool IsInteger(EPixelFormat PixelFormat)
{
	PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	switch (PixelFormat)
	{
	case PF_R32_UINT:
	case PF_R32_SINT:
	case PF_R16_UINT:
	case PF_R16_SINT:
	case PF_R16G16B16A16_UINT:
	case PF_R16G16B16A16_SINT:
	case PF_R32G32B32A32_UINT:
	case PF_R16G16_UINT:
	case PF_R8_UINT:
	case PF_R8G8B8A8_UINT:
	case PF_R32G32_UINT:
	case PF_R8G8_UINT:
	case PF_R32G32B32_UINT:
	case PF_R32G32B32_SINT:
	case PF_R8_SINT:
	case PF_R64_UINT:
	case PF_R16G16_SINT:
		return true;
	}
	PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	return false;
}

// IsFloatFormat does not include all floating point formats; see also IsHDR
inline bool IsFloatFormat(EPixelFormat Format)
{
	PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	switch (Format)
	{
	case PF_A32B32G32R32F:
	case PF_FloatRGB:
	case PF_FloatRGBA:
	case PF_R32_FLOAT:
	case PF_G16R16F:
	case PF_G16R16F_FILTER:
	case PF_G32R32F:
	case PF_R16F:
	case PF_R16F_FILTER:
	case PF_FloatR11G11B10:
		return true;
	}
	PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	return false;
}

inline bool IsDepthOrStencilFormat(EPixelFormat Format)
{
	PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	switch (Format)
	{
	case PF_D24:
	case PF_DepthStencil:
	case PF_X24_G8:
	case PF_ShadowDepth:
	case PF_R32_FLOAT:
		return true;
	}
	PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	return false;
}

inline bool IsStencilFormat(EPixelFormat Format)
{
	PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	switch (Format)
	{
	case PF_DepthStencil:
	case PF_X24_G8:
		return true;
	}
	PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	return false;
}

inline bool IsDXTCBlockCompressedTextureFormat(EPixelFormat PixelFormat)
{
	PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	switch (PixelFormat)
	{
	case PF_DXT1:
	case PF_DXT3:
	case PF_DXT5:
	case PF_BC4:
	case PF_BC5:
	case PF_BC6H:
	case PF_BC7:
		return true;
	default:
		return false;
	}
	PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
}

UE_FORCEINLINE_HINT bool RequiresBlock4Alignment(EPixelFormat PixelFormat)
{
	// BCN DXTC formats require 4x4 alignment, but ASTC/ETC do not
	return IsDXTCBlockCompressedTextureFormat(PixelFormat);
}

/** 
* Information about a pixel format. The majority of this structure is valid after static init, however RHI does keep
* some state in here that is initialized by that module and should not be used by general use programs that don't
* have RHI (so noted in comments).
*/
struct FPixelFormatInfo
{
	FPixelFormatInfo() = delete;
	constexpr FPixelFormatInfo(
		EPixelFormat InUnrealFormat,
		const TCHAR* InName,
		int32 InBlockSizeX,
		int32 InBlockSizeY,
		int32 InBlockSizeZ,
		int32 InBlockBytes,
		int32 InNumComponents,
		bool  InSupported);

	const TCHAR*				Name;
	EPixelFormat				UnrealFormat;
	int32						BlockSizeX;
	int32						BlockSizeY;
	int32						BlockSizeZ;
	int32						BlockBytes;
	int32						NumComponents;

	/** Per platform cabilities for the format (initialized by RHI module - invalid otherwise) */
	EPixelFormatCapabilities	Capabilities = EPixelFormatCapabilities::None;

	/** Platform specific converted format (initialized by RHI module - invalid otherwise) */
	uint32						PlatformFormat{ 0 };

	/** Whether the texture format is supported on the current platform/ rendering combination */
	uint8						Supported : 1;

	// If false, 32 bit float is assumed (initialized by RHI module - invalid otherwise)
	uint8						bIs24BitUnormDepthStencil : 1;
	
	/** 
	* Get 2D/3D image/texture size in bytes. This is for storage of the encoded image data, and does not adjust
	* for any GPU alignment/padding constraints. It is also not valid for tiled or packed mip tails (i.e. cooked mips 
	* for consoles). Only use these when you know you're working with bog standard textures/images in block based pixel formats.
	*/
	CORE_API uint64 Get2DImageSizeInBytes(uint32 InWidth, uint32 InHeight) const;
	CORE_API uint64 Get2DTextureMipSizeInBytes(uint32 InTextureWidth, uint32 InTextureHeight, uint32 InMipIndex) const;
	CORE_API uint64 Get2DTextureSizeInBytes(uint32 InTextureWidth, uint32 InTextureHeight, uint32 InMipCount) const;
	CORE_API uint64 Get3DImageSizeInBytes(uint32 InWidth, uint32 InHeight, uint32 InDepth) const;
	CORE_API uint64 Get3DTextureMipSizeInBytes(uint32 InTextureWidth, uint32 InTextureHeight, uint32 InTextureDepth, uint32 InMipIndex) const;
	CORE_API uint64 Get3DTextureSizeInBytes(uint32 InTextureWidth, uint32 InTextureHeight, uint32 InTextureDepth, uint32 InMipCount) const;
	
	/**
	* Get the number of compressed blocks necessary to hold the given dimensions.
	*/
	CORE_API uint64 GetBlockCountForWidth(uint32 InWidth) const;
	CORE_API uint64 GetBlockCountForHeight(uint32 InHeight) const;
};


/**
 * enum to string
 * Note this is not the FPixelFormatInfo::Name
 * @return e.g. "PF_B8G8R8A8"
 */
CORE_API const TCHAR* GetPixelFormatString(EPixelFormat InPixelFormat);
/**
 * string to enum (not case sensitive)
 * Note this is not the FPixelFormatInfo::Name
 * @param InPixelFormatStr e.g. "PF_B8G8R8A8", must not not be 0
 */
CORE_API EPixelFormat GetPixelFormatFromString(const TCHAR* InPixelFormatStr);


extern CORE_API FPixelFormatInfo GPixelFormats[PF_MAX];		// Maps members of EPixelFormat to a FPixelFormatInfo describing the format.

namespace UE::PixelFormat
{
	inline bool HasCapabilities(EPixelFormat InFormat, EPixelFormatCapabilities InCapabilities)
	{
		return EnumHasAllFlags(GPixelFormats[InFormat].Capabilities, InCapabilities);
	}
}
