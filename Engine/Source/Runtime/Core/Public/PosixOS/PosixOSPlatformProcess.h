// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "GenericPlatform/GenericPlatformProcess.h"


// This class can be used as a "mixin" or subclassed from - for instance
// Linux can fully use (subclass from) this but Mac may not have all Posix 
// functionality, and it subclasses from Apple, which is shared with IOS 
// (which isn't in the PosixOS platform group)
struct FPosixOSPlatformProcess : public FGenericPlatformProcess
{
	/** FPlatformProcess API */
	static CORE_API bool IsFirstInstance();
	static CORE_API void CeaseBeingFirstInstance();
	static CORE_API TSharedPtr<IProcessSentinel> CreateProcessSentinelObject();
};
