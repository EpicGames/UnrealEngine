// Copyright Epic Games, Inc. All Rights Reserved.

/**
* Declarations for ABTesting framework.
*/

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Math/RandomStream.h"
#include "Misc/Build.h"

#if !UE_BUILD_SHIPPING
#define ENABLE_ABTEST 1
#else
#define ENABLE_ABTEST 0
#endif

#include "ProfilingDebugging/ScopedTimers.h"

template<typename Allocator > class TBitArray;

#if ENABLE_ABTEST
class FABTest
{
public:

	CORE_API FABTest();
	
	//returns a command to execute, if any.
	CORE_API const TCHAR* TickAndGetCommand();

	bool IsActive()
	{
		return bABTestActive;
	}

	void ReportScopeTime(double ScopeTime)
	{
		TotalScopeTimeInFrame += ScopeTime;
	}

	bool GetDoFirstScopeTest() const
	{
		return CurrentTest == 0;
	}

	static CORE_API FABTest& Get();

	static CORE_API void ABTestCmdFunc(const TArray<FString>& Args);

	static bool StaticIsActive()
	{
		return Get().bABTestActive;
	}

private:

	CORE_API void StartFrameLog();
	CORE_API void FrameLogTick(double Delta);
	
	CORE_API void Start(FString* InABTestCmds, bool bScopeTest);
	CORE_API void Stop();

	CORE_API const TCHAR* SwitchTest(int32 Index);
	
	

	FRandomStream Stream;
	bool bABTestActive;
	bool bABScopeTestActive; //whether we are doing abtesting on the scope macro rather than a CVAR.	
	bool bFrameLog;
	FString ABTestCmds[2];
	int32 ABTestNumSamples;
	int32 RemainingCoolDown;
	int32 CurrentTest;
	int32 RemainingTrial;
	int32 RemainingPrint;

	int32 HistoryNum;
	int32 ReportNum;
	int32 CoolDown;
	int32 MinFramesPerTrial;
	int32 NumResamples;

	struct FSample
	{
		uint32 Micros;
		int32 TestIndex;
		TBitArray<> InResamples;
	};

	TArray<FSample> Samples;
	TArray<uint32> ResampleAccumulators;
	TArray<uint32> ResampleCount;
	uint32 Totals[2];
	uint32 Counts[2];

	double TotalTime;
	uint32 TotalFrames;
	uint32 Spikes;

	double TotalScopeTimeInFrame;
	uint64 LastGCFrame;

};

class FScopedABTimer : public FDurationTimer
{
public:
	explicit FScopedABTimer() 
		: FDurationTimer(TimerData)
		, TimerData(0)
	{
		Start();
	}

	/** Dtor, updating seconds with time delta. */
	~FScopedABTimer()
	{
		Stop();
		FABTest::Get().ReportScopeTime(Accumulator);
	}
private:
	double TimerData;
};

#define SCOPED_ABTEST() FScopedABTimer ABSCOPETIMER;
#define SCOPED_ABTEST_DOFIRSTTEST() FABTest::Get().GetDoFirstScopeTest()
#else

class FABTest
{
public:
	static bool StaticIsActive()
	{
		return false;
	}
};

#define SCOPED_ABTEST(TimerName)
#define SCOPED_ABTEST_DOFIRSTTEST() true
#endif
