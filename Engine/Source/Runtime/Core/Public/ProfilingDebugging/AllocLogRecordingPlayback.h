// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AllocLogRecordingPlayback.h: Allocation recording and playback to/from log files
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Misc/FileHelper.h"

/*
 * Representation of an alloc or free operation. Derive from this struct to add
 * items such as size and alignment. Override the three methods to log or parse
 * the values of custom items. A new line character must be added to the end once
 * all items have been written to the string buffer.
 */
struct FAllocOpBase
{
	int64 AllocIndex;
	int32 FrameNumber;
	bool bIsFree;

	static int32 WriteHeader(ANSICHAR* Dest)
	{
		return FCStringAnsi::Sprintf(Dest, "Operation,FrameNumber,AllocIndex");
	}

	int32 WriteValueStrings(ANSICHAR* Dest) const
	{
		return FCStringAnsi::Sprintf(Dest, "%s,%d,%lld", bIsFree ? "Free" : "Alloc", FrameNumber, AllocIndex);
	}

	int32 ParseValueStrings(const TArray<FString>& ValueStrings)
	{
		if (ValueStrings.Num() < 3)
		{
			return -1;
		}

		if (ValueStrings[0] == TEXT("Alloc"))
		{
			bIsFree = false;
		}
		else if (ValueStrings[0] == TEXT("Free"))
		{
			bIsFree = true;
		}
		else
		{
			return -1;
		}

		FrameNumber = FCString::Atoi(*ValueStrings[1]);
		AllocIndex = FCString::Atoi64(*ValueStrings[2]);
		return 3;
	}
};

/*
 * Define a modifier class like below and supply it to TAllocRecordingLog to complete the template.
 * Create an instance of the completed TAllocRecordingLog before any allocation of interest is done.
 * Inject RecordAlloc and RecordFree to your allocator wherever appropriate.
 * 
 * class FModifier
 * {
 *   // Initialize custom member variables
 *   FModifier(...);
 * 
 *   bool ShouldRecordAlloc(const FAllocOp& Op) const;
 * 
 *   static void HandleFatalError(const TCHAR* Message);
 *
 *   static void HandleDisplayMessage(const TCHAR* Message);
 * };
 */
template <typename FModifier, typename FAllocOp>
class TAllocRecordingLog : public FModifier
{
protected:
	using FModifier::ShouldRecordAlloc;
	using FModifier::HandleFatalError;
	using FModifier::HandleDisplayMessage;

	void WriteLineBuffer(int32 NumCharacters)
	{
		if (NumCharacters > 0 && NumCharacters < MAX_SPRINTF)
		{
			OutputFile->Serialize((void*)LineBuffer, NumCharacters);
		}
	}

	int64 CurrentAllocationIndex;
	FCriticalSection CS;
	TSharedPtr<FArchive> OutputFile;

	ANSICHAR LineBuffer[MAX_SPRINTF];

public:
	template <typename... ArgTypes>
	TAllocRecordingLog(const FString& Filename, ArgTypes&&... Args)
		: FModifier(Forward<ArgTypes>(Args)...)
		, CurrentAllocationIndex(0)
	{
		OutputFile = MakeShareable(IFileManager::Get().CreateFileWriter(*Filename));
		if (!OutputFile)
		{
			HandleFatalError(*FString::Printf(TEXT("Failed to create allocation log file \"%s\". Capture will not start."), *Filename));
		}
		HandleDisplayMessage(*FString::Printf(TEXT("Recording allocations to file: %s"), *Filename));
		WriteLineBuffer(FAllocOp::WriteHeader(LineBuffer));
	}

	template <typename... ArgTypes>
	int64 RecordAlloc(ArgTypes&&... Args)
	{
		FAllocOp Op(Forward<ArgTypes>(Args)...);
		Op.bIsFree = false;
		if (ShouldRecordAlloc(Op))
		{
			FScopeLock Lock(&CS);
			Op.AllocIndex = CurrentAllocationIndex++;
			Op.FrameNumber = GFrameNumberRenderThread;
			WriteLineBuffer(Op.WriteValueStrings(LineBuffer));
			return Op.AllocIndex;
		}
		return INDEX_NONE;
	}

	void RecordFree(int64 AllocIndex)
	{
		if (AllocIndex != INDEX_NONE)
		{
			FScopeLock Lock(&CS);
			FAllocOp Op;
			Op.bIsFree = true;
			Op.AllocIndex = AllocIndex;
			Op.FrameNumber = GFrameNumberRenderThread;
			WriteLineBuffer(Op.WriteValueStrings(LineBuffer));
		}
	}
};

/*
 * Define a modifier class like below and supply it to TAllocPlaybackLog to complete the template.
 * Create an instance of the completed TAllocPlaybackLog. Call Load to load an allocation log
 * generated by your specialized TAllocRecordingLog. Call Play to playback recorded allocations
 * and frees. Record and write out profiling data as you see fit.
 * 
 * class FModifier
 * {
 *   typedef <...> FAllocType;
 * 
 *   // Initialize custom member variables
 *   FModifier(...);
 * 
 *   int32 OnPlaybackStart();
 *   void OnPlaybackEnd(int32 NumAllocOpsReplayed);
 * 
 *   void OnNewPlaybackFrame();
 *   void OnEndPlaybackFrame();
 * 
 *   void OnFree(const FAllocType& Allocation);
 *   FAllocType OnAlloc(const FAllocOp& Op);
 * 
 *   static bool IsValidAllocation(const FAllocType& Allocation);
 * 
 *   static void HandleFatalError(const TCHAR* Message);
 *   static void HandleErrorMessage(const TCHAR* Message);
 * };
 */
template <typename FModifier, typename FAllocOp>
class TAllocPlaybackLog : public FModifier
{
protected:
	using FFilterType = typename FAllocOp::FFilterType;
	using FAllocType = typename FModifier::FAllocType;

	using FModifier::HandleFatalError;
	using FModifier::HandleErrorMessage;
	using FModifier::OnPlaybackStart;
	using FModifier::OnPlaybackEnd;
	using FModifier::OnNewPlaybackFrame;
	using FModifier::OnEndPlaybackFrame;
	using FModifier::OnFree;
	using FModifier::OnAlloc;
	using FModifier::IsValidAllocation;

	FFilterType CurrentFilter;
	TArray<FAllocOp> Ops;

public:
	template <typename... ArgTypes>
	TAllocPlaybackLog(FFilterType InFilter, ArgTypes&&... Args)
		: FModifier(Forward<ArgTypes>(Args)...)
		, CurrentFilter(InFilter)
	{}

	void Load(FString& Filename)
	{
		TArray<FString> AllocLog;
		if (!FFileHelper::LoadFileToStringArray(AllocLog, *Filename))
		{
			HandleFatalError(*FString::Printf(TEXT("Failed to load alloc log %s"), *Filename));
		}

		TMap<int64, FFilterType> AllocFilters;
		for (int32 LineIndex = 1; LineIndex < AllocLog.Num(); ++LineIndex)
		{
			const FString& Line = AllocLog[LineIndex];
			TArray<FString> ValueStrings;
			Line.ParseIntoArray(ValueStrings, TEXT(","));

			FAllocOp Op;
			if (Op.ParseValueStrings(ValueStrings) < 0)
			{
				continue;
			}

			if (Op.bIsFree)
			{
				// Check there's a corresponding allocation and get the filter
				const FFilterType* FilterFind = AllocFilters.Find(Op.AllocIndex);
				if (!FilterFind)
				{
					// Filter this free out. There is no preceding corresponding alloc
					continue;
				}
				Op.SetFilter(*FilterFind);
			}
			else
			{
				AllocFilters.Add(Op.AllocIndex, Op.GetFilter());
			}
			
			// Check filter
			if (Op.MatchFilter(CurrentFilter))
			{
				Ops.Add(Op);
			}
		}
	}

	void Play(int32 MaxOpCount = INT32_MAX)
	{	
		const int32 NumIterations = OnPlaybackStart();
		const int32 OpCount = FMath::Min(MaxOpCount, Ops.Num());
		bool bStartNewFrame = true;
		TMap<int64, FAllocType> AllocMap;

		for (int32 Iteration = 0; Iteration < NumIterations; ++Iteration)
		{
			for (int32 OpIndex = 0; OpIndex < OpCount; ++OpIndex)
			{
				const FAllocOp& Op = Ops[OpIndex];

				if (bStartNewFrame)
				{
					OnNewPlaybackFrame();
					bStartNewFrame = false;
				}

				if (Op.bIsFree)
				{
					FAllocType* AllocEntry = AllocMap.Find(Op.AllocIndex);
					if (AllocEntry)
					{
						OnFree(*AllocEntry);
						AllocMap.Remove(Op.AllocIndex);
					}
				}
				else
				{
					FAllocType Allocation = OnAlloc(Op);
					if (IsValidAllocation(Allocation))
					{
						if (!AllocMap.Find(Op.AllocIndex))
						{
							AllocMap.Add(Op.AllocIndex, MoveTemp(Allocation));
						}
						else
						{
							OnFree(Allocation);
							HandleErrorMessage(*FString::Printf(TEXT("Duplicated allocation with index %lld found. This should not happen. The log may be corrupted."), Op.AllocIndex));
						}
					}
				}

				// If the last op of the frame, end the frame
				int32 NextFrameNumber = OpIndex < OpCount - 1 ? Ops[OpIndex + 1].FrameNumber : -1;
				if (Op.FrameNumber != NextFrameNumber)
				{
					OnEndPlaybackFrame();
					bStartNewFrame = true;
				}
			}
		}
		
		OnPlaybackEnd(OpCount);
	}
};
