// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MetadataTrace.h"
#include "ProfilingDebugging/StringsTrace.h"
#include "Trace/Detail/EventNode.h"
#include "Trace/Trace.h"

namespace UE { namespace Trace { class FChannel; } }

#if !defined(UE_TRACE_ASSET_METADATA_ENABLED)
	#define UE_TRACE_ASSET_METADATA_ENABLED UE_TRACE_METADATA_ENABLED
#endif

#if UE_TRACE_ASSET_METADATA_ENABLED

/**
 * Channel that asset metadata is output on
 */
CORE_API UE_TRACE_CHANNEL_EXTERN(AssetMetadataChannel);

/**
 * Metadata scope to instrument operations belonging to a certain asset
 */
CORE_API UE_TRACE_METADATA_EVENT_BEGIN_EXTERN(Asset)
	UE_TRACE_METADATA_EVENT_REFERENCE_FIELD(Strings, FName, Name)
	UE_TRACE_METADATA_EVENT_REFERENCE_FIELD(Strings, FName, Class)
	UE_TRACE_METADATA_EVENT_REFERENCE_FIELD(Strings, FName, Package)
UE_TRACE_METADATA_EVENT_END()

/**
 * Metadata scope to instrument operations belonging to a certain package
 */
CORE_API UE_TRACE_METADATA_EVENT_BEGIN_EXTERN(PackageId)
	UE_TRACE_METADATA_EVENT_FIELD(uint64, Id)
UE_TRACE_METADATA_EVENT_END()

/**
 * Metadata event to record the package name for a given package id. The package name may not be known until after the package is loaded
 */
CORE_API UE_TRACE_EVENT_BEGIN_EXTERN(Package, PackageMapping, NoSync)
	UE_TRACE_EVENT_FIELD(uint64, Id)
	UE_TRACE_EVENT_REFERENCE_FIELD(Strings, FName, Package)
UE_TRACE_EVENT_END()

/**
 * Utility macro to create asset scope from an object and object class
 * @note When using this macro in a module outside of Core, make sure a dependency to the "TraceLog" module is added.
 */
#define UE_TRACE_METADATA_SCOPE_ASSET(Object, ObjClass) \
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(Object->GetFName(), ObjClass->GetFName(), Object->GetPackage()->GetFName())

/**
 * Utility macro to create an asset scope by specifying object name, class name and package name explicitly
 */
#define UE_TRACE_METADATA_SCOPE_ASSET_FNAME(ObjectName, ObjClassName, PackageName) \
	auto MetaNameRef = bool(MetadataChannel) && bool(AssetMetadataChannel) ? FStringTrace::GetNameRef(ObjectName) : UE::Trace::FEventRef32(0,0); \
	auto ClassNameRef = bool(MetadataChannel) && bool(AssetMetadataChannel) ? FStringTrace::GetNameRef(ObjClassName) : UE::Trace::FEventRef32(0,0); \
	auto PackageNameRef = bool(MetadataChannel) && bool(AssetMetadataChannel) ? FStringTrace::GetNameRef(PackageName) : UE::Trace::FEventRef32(0,0); \
	UE_TRACE_METADATA_SCOPE(Asset, AssetMetadataChannel) \
		<< Asset.Name(MetaNameRef) \
		<< Asset.Class(ClassNameRef) \
		<< Asset.Package(PackageNameRef);

/*
 * Utility macro to create a package scope by specifying package id explicitly
 */
#define UE_TRACE_METADATA_SCOPE_PACKAGE_ID(UPackageId) \
	UE_TRACE_METADATA_SCOPE(PackageId, AssetMetadataChannel) \
		<< PackageId.Id(UPackageId.Value());

/**
 * Utility macro to record the name of a given package, if known
 */
#define UE_TRACE_PACKAGE_NAME(UPackageId, PackageName) \
	if (!PackageName.IsNone()) \
	{ \
		auto PackageNameRef = bool(MetadataChannel) && bool(AssetMetadataChannel) ? FStringTrace::GetNameRef(PackageName) : UE::Trace::FEventRef32(0,0); \
		UE_TRACE_LOG(Package, PackageMapping, MetadataChannel | AssetMetadataChannel) \
			<< PackageMapping.Id(UPackageId.Value()) \
			<< PackageMapping.Package(PackageNameRef); \
	}


#else // UE_TRACE_ASSET_METADATA_ENABLED

#define UE_TRACE_METADATA_SCOPE_ASSET(...)
#define UE_TRACE_METADATA_SCOPE_ASSET_FNAME(...)

#define UE_TRACE_PACKAGE_NAME(...)
#define UE_TRACE_METADATA_SCOPE_PACKAGE_ID(...)

#endif // UE_TRACE_ASSET_METADATA_ENABLED
