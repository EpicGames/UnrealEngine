// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TODO: Consider moving boot timing stuff from CoreGlobals.h here in UE5 - involves adding a new include to 50+ files

struct FBootProfiling
{
	static CORE_API double GetBootDuration();
	static CORE_API double GetPreInitPreStartupScreenDuration();
	static CORE_API double GetPreInitPostStartupScreenDuration();
	static CORE_API double GetEngineInitDuration();

	enum class ECounterPhase
	{
		DuringEngineInit,
		PostEngineInit,
		AllPhases,	// combine all phases above
	};

	// These are to be called at the appropriate time during EngineInit
	static CORE_API void InitCounters();
	static CORE_API void OnPreInitPreStartupScreenComplete();
	static CORE_API void OnPreInitPostStartupScreenComplete();
	static CORE_API void OnInitComplete();

	/** Accumulate time spent in blocking dialogs (e.g. driver denylist) during boot. */
	static CORE_API void AddBlockingDialogDuration(double Seconds);

	/** Returns total time spent in blocking dialogs during boot. */
	static CORE_API double GetBlockingDialogDuration();

	/* Returns the amount of times the app has been suspended (mobile/switch) during the specified phase.
	 * NOTE: returns -1 when an incorrect phase is passed.
	 */
	static CORE_API int GetAppSuspendedCounter(ECounterPhase phase);

	/* Returns the amount of times the app has been activated (mobile/switch) during the specified phase. 
	 * NOTE: returns -1 when an incorrect phase is passed.
	 */
	static CORE_API int GetAppActivatedCounter(ECounterPhase phase);
};
