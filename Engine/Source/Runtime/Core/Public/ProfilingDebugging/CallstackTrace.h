// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/Defines.h"
#include "HAL/Platform.h"
#include "HAL/PlatformTLS.h"
#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "Serialization/VarInt.h"

////////////////////////////////////////////////////////////////////////////////
#if !defined(UE_CALLSTACK_TRACE_ENABLED)
	#if UE_TRACE_ENABLED && !UE_BUILD_SHIPPING
		#if PLATFORM_WINDOWS
			#define UE_CALLSTACK_TRACE_ENABLED 1
		#endif
	#endif
#endif

#if !defined(UE_CALLSTACK_TRACE_ENABLED)
	#define UE_CALLSTACK_TRACE_ENABLED 0
#endif

#if !defined(UE_CALLSTACK_TRACE_MAX_FRAMES)
	#define UE_CALLSTACK_TRACE_MAX_FRAMES 255
#endif

////////////////////////////////////////////////////////////////////////////////
#if UE_CALLSTACK_TRACE_ENABLED

/**
 * Creates callstack tracing.
 * @param Malloc Allocator instance to use.
 */
void CallstackTrace_Create(class FMalloc* Malloc);

/**
 * Initializes callstack tracing. On some platforms this has to be delayed due to initialization order.
 */
void CallstackTrace_Initialize();

/**
 * Filter modules that should be considered for callstack tracing. Only used on
 * platforms that supports custom stack walking. See platform implementations for
 * details.
 * @param ModuleName Name of the module to be filtered
 * @return True if module should ignored for stack walking, false otherwise
 */
bool CallstackTrace_FilterModule(FStringView ModuleName);

/**
 * Capture the current callstack, and trace the definition if it has not already been encountered. The returned value
 * can be used in trace events and be resolved in analysis.
 * @return Unique id identifying the current callstack.
 */
AUTORTFM_OPEN CORE_API uint32 CallstackTrace_GetCurrentId();

 /**
  * Callstack Trace Scoped Macro to avoid resolving the full callstack
  * can be used when some external libraries are not compiled with frame pointers
  * preventing us to resolve it without crashing. Instead the callstack will be
  * only the caller address.
  */
#define CALLSTACK_TRACE_LIMIT_CALLSTACKRESOLVE_SCOPE() \
	FCallStackTraceLimitResolveScope UE_JOIN(FCTLMScope,__LINE__)

extern CORE_API uint32 GCallStackTracingTlsSlotIndex;

/**
* @return the fallback callstack address
*/
inline void* CallstackTrace_GetFallbackPlatformReturnAddressData()
{
	if (FPlatformTLS::IsValidTlsSlot(GCallStackTracingTlsSlotIndex))
	{
		return FPlatformTLS::GetTlsValue(GCallStackTracingTlsSlotIndex);
	}
	else
	{
		return nullptr;
	}
}

/**
* @return Needs full callstack resolve
*/
inline bool CallstackTrace_ResolveFullCallStack()
{
	return CallstackTrace_GetFallbackPlatformReturnAddressData() == nullptr;
}

/*
 * Callstack Trace scope for override CallStack
 */
class FCallStackTraceLimitResolveScope
{
public:
	FORCENOINLINE FCallStackTraceLimitResolveScope()
	{
		if (FPlatformTLS::IsValidTlsSlot(GCallStackTracingTlsSlotIndex))
		{
			FPlatformTLS::SetTlsValue(GCallStackTracingTlsSlotIndex, PLATFORM_RETURN_ADDRESS_FOR_CALLSTACKTRACING());
		}
	}

	FORCENOINLINE ~FCallStackTraceLimitResolveScope()
	{
		if (FPlatformTLS::IsValidTlsSlot(GCallStackTracingTlsSlotIndex))
		{
			FPlatformTLS::SetTlsValue(GCallStackTracingTlsSlotIndex, nullptr);
		}
	}
};

#else // UE_CALLSTACK_TRACE_ENABLED

inline void CallstackTrace_Create(class FMalloc* Malloc) {}
inline void CallstackTrace_Initialize() {}
AUTORTFM_OPEN inline uint32 CallstackTrace_GetCurrentId() { return 0; }
inline void* CallstackTrace_GetCurrentReturnAddressData() { return nullptr; }
inline void* CallstackTrace_GetFallbackPlatformReturnAddressData() { return nullptr; }
inline bool  CallstackTrace_ResolveFullCallStack() { return true; }

#define CALLSTACK_TRACE_LIMIT_CALLSTACKRESOLVE_SCOPE()

#endif // UE_CALLSTACK_TRACE_ENABLED

/**
 * Simple run length compression of arrays of addresses (e.g stack frames).
 * Needs to be available regardless if tracing is enabled to allow compress/decompress analysis.
 */
namespace FCallstackXORAndRLE
{
	inline uint32 Compress(TConstArrayView<uint64> Addresses, TArrayView<uint8> Compressed)
	{
		check(Addresses.Num() * 8 <= Compressed.Num());
		uint32 CompressedBytes = 0;
		uint64 LastAddress = 0;
		for(uint8 Index = 0; Index < Addresses.Num(); ++Index)
		{
			const uint64 Address = Addresses[Index];
			uint64 XorAddress = LastAddress ^ Address;
			LastAddress = Address;
			const uint8 LeadingZeroes = uint8(FMath::CountLeadingZeros64(XorAddress));
			Compressed[CompressedBytes++] = LeadingZeroes;
			if (LeadingZeroes < 64)
			{
				int8 ValueBits = 64 - LeadingZeroes;
				do
				{
					Compressed[CompressedBytes++] = uint8(0xff & XorAddress);
					XorAddress >>= 8;
					ValueBits -= 8;
				} while(ValueBits > 0);
			}
		}
		return CompressedBytes;
	}

	inline uint32 Uncompress(TConstArrayView<uint8> Compressed, TArrayView<uint64> Addresses, uint32& OutTotalAddressCount)
	{
		const int32 Bytes = Compressed.Num();
		int32 ByteIndex = 0;
		uint64 LastAddress = 0;
		while (ByteIndex < Bytes)
		{
			uint64 OutAddress = 0;
			const uint8 LeadingZeroes = Compressed[ByteIndex++];
			if (LeadingZeroes < 64)
			{
				int8 ValueBits = 64 - LeadingZeroes;
				uint8 BitsShift = 0;
				check(ValueBits <= (Bytes - ByteIndex)*8);
				while (ValueBits > 0)
				{
					OutAddress = (uint64(Compressed[ByteIndex++]) << BitsShift) | OutAddress;
					ValueBits -= 8;
					BitsShift += 8;
				}
			}
			OutAddress = LastAddress ^ OutAddress;
			LastAddress = OutAddress;

			if (OutTotalAddressCount < (uint32)Addresses.Num())
			{
				Addresses[OutTotalAddressCount] = OutAddress;
			}
			OutTotalAddressCount++;
		}
		return FMath::Min(OutTotalAddressCount, (uint32)Addresses.Num());
	}
} // namespace FCallstackXORAndRLE

/**
 * Delta + ZigZag encoding with 7bit compression of arrays of addresses (e.g stack frames).
 * Needs to be available regardless if tracing is enabled to allow compress/decompress analysis.
 */
namespace FCallstackDelta7bit
{
	inline uint32 Compress(TConstArrayView<uint64> Addresses, TArrayView<uint8> Compressed)
	{
		check(Addresses.Num() * 10 <= Compressed.Num());
		uint32 CompressedBytes = 0;
		uint64 LastAddress = 0;
		for(uint8 Index = 0; Index < Addresses.Num(); ++Index)
		{
			const uint64 Address = Addresses[Index];
			const int64 Delta = int64(Address - LastAddress);
			LastAddress = Address;

			uint64 EncodedAddress = (uint64(Delta) << 1) ^ uint64(Delta >> 63);
			do
			{
				uint8 Byte = EncodedAddress & 0x7F;
				EncodedAddress >>= 7;
				Compressed[CompressedBytes++] = Byte | uint8((EncodedAddress > 0) << 7);
			}
			while (EncodedAddress > 0);
		}
		return CompressedBytes;
	}

	inline uint32 Uncompress(TConstArrayView<uint8> Compressed, TArrayView<uint64> Addresses, uint32& OutTotalAddressCount)
	{
		const int32 Bytes = Compressed.Num();
		int32 ByteIndex = 0;
		uint64 LastAddress = 0;
		while (ByteIndex < Bytes)
		{
			uint64 EncodedAddress = 0;
			int Shift = 0;
			uint8 Byte = 0;
			do
			{
				Byte = Compressed[ByteIndex++];
				EncodedAddress |= uint64(Byte & 0x7F) << Shift;
				Shift += 7;
			} while (Byte & 0x80);

			int64 Delta = (EncodedAddress >> 1) ^ -(int64)(EncodedAddress & 1);
			uint64 Address = LastAddress + Delta;
			LastAddress = Address;

			if (OutTotalAddressCount < (uint32)Addresses.Num())
			{
				Addresses[OutTotalAddressCount] = Address;
			}
			OutTotalAddressCount++;
		}
		return FMath::Min(OutTotalAddressCount, (uint32)Addresses.Num());
	}
} // namespace FCallstackDelta7bit

/**
 * Delta + ZigZag encoding with VarInt compression of arrays of addresses (e.g stack frames).
 * Needs to be available regardless if tracing is enabled to allow compress/decompress analysis.
 */
namespace FCallstackDeltaVarInt
{
	inline uint32 Compress(TConstArrayView<uint64> Addresses, TArrayView<uint8> Compressed)
	{
		check(Addresses.Num() * 9 <= Compressed.Num());
		uint8* CompressedPtr = Compressed.GetData();
		uint64 LastAddress = 0;
		for(uint8 Index = 0; Index < Addresses.Num(); ++Index)
		{
			const uint64 Address = Addresses[Index];
			int64 Delta = int64(Address - LastAddress);
			LastAddress = Address;

			uint32 BytesCompressed = WriteVarInt(Delta, CompressedPtr);
			CompressedPtr += BytesCompressed;
		}
		return uint32(CompressedPtr - Compressed.GetData());
	}

	inline uint32 Uncompress(TConstArrayView<uint8> Compressed, TArrayView<uint64> Addresses, uint32& OutTotalAddressCount)
	{
		const uint8* CompressedPtr = Compressed.GetData();
		const uint8* EndPtr = CompressedPtr + Compressed.Num();
		uint64 Address = 0;
		while (CompressedPtr < EndPtr)
		{
			uint32 BytesFound = 0;
			int64 Delta = ReadVarInt(CompressedPtr, BytesFound);
			CompressedPtr += BytesFound;
			Address += Delta;

			if (OutTotalAddressCount < (uint32)Addresses.Num())
			{
				Addresses[OutTotalAddressCount] = Address;
			}
			OutTotalAddressCount++;
		}
		return FMath::Min(OutTotalAddressCount, (uint32)Addresses.Num());
	}
} // namespace FCallstackDeltaVarInt
