// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformTime.h"
#include "HAL/PreprocessorHelpers.h"
#include "HAL/ThreadSafeCounter64.h"
#include "Templates/Decay.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Trace.h"

namespace UE { namespace Trace { class FChannel; } }

#ifndef ENABLE_COOK_STATS
#define ENABLE_COOK_STATS WITH_EDITOR
#endif

template <typename FuncType> class TFunctionRef;
template <typename T> struct TDecay;

enum class EPackageEventStatType : uint8
{
	LoadPackage = 0,
	SavePackage = 1,
	BeginCacheForCookedPlatformData = 2,
	IsCachedCookedPlatformDataLoaded = 3,
};

#if ENABLE_COOK_STATS
/**
 * Centralizes the system to gather stats from a cook that need to be collected at the core/engine level.
 * Essentially, add a delegate to the CookStatsCallbacks member. When a cook a complete that is configured to use stats (ENABLE_COOK_STATS),
 * it will broadcast this delegate, which, when called, gives you a TFunction to call to report each of your stats.
 * 
 * For simplicity, FAutoRegisterCallback is provided to auto-register your callback on startup. Usage is like:
 * 
 * static void ReportCookStats(AddStatFuncRef AddStat)
 * {
 *		AddStat("MySubsystem.Event1", CreateKeyValueArray("Attr1", "Value1", "Attr2", "Value2" ... );
 *		AddStat("MySubsystem.Event2", CreateKeyValueArray("Attr1", "Value1", "Attr2", "Value2" ... );
 * }
 * 
 * FAutoRegisterCallback GMySubsystemCookRegistration(&ReportCookStats);
 * 
 * When a cook is complete, your callback will be called, and the stats will be either logged, sent to an
 * analytics provider, logged to an external file, etc. You don't care how they are added, you just call 
 * AddStat for each stat you want to add.
 */
class FCookStatsManager
{
public:
	/** 
	 * Copy of TKeyValuePair<> from Core, but with a default initializer for each member. Useful for containers that auto-add default constructed members. 
	 * We can't change TKeyValuePair because code relies on it and don't want to slow it down for those use cases.
	 */
	template <typename KeyType, typename ValueType>
	struct TKeyValuePair
	{
		template<typename KeyType2, typename ValueType2>
		TKeyValuePair(KeyType2&& InKey, ValueType2&& InValue)
		: Key(Forward<KeyType2>(InKey)), Value(Forward<ValueType2>(InValue))
		{
		}
		TKeyValuePair() : Key {}, Value {}
		{
		}
		bool operator==(const TKeyValuePair& Other) const
		{
			return Key == Other.Key;
		}
		bool operator!=(const TKeyValuePair& Other) const
		{
			return Key != Other.Key;
		}
		bool operator<(const TKeyValuePair& Other) const
		{
			return Key < Other.Key;
		}
		UE_FORCEINLINE_HINT bool operator()(const TKeyValuePair& A, const TKeyValuePair& B) const
		{
			return A.Key < B.Key;
		}
		KeyType		Key;
		ValueType	Value;
	};

	/** Helper to construct a TKeyValuePair. */
	template <typename KeyType, typename ValueType>
	static TKeyValuePair<typename TDecay<KeyType>::Type, typename TDecay<ValueType>::Type> MakePair(KeyType&& InKey, ValueType&& InValue)
	{
		return TKeyValuePair<typename TDecay<KeyType>::Type, typename TDecay<ValueType>::Type>(Forward<KeyType>(InKey), Forward<ValueType>(InValue));
	}


	/** Every stat is essentially a name followed by an array of key/value "attributes" associated with the stat. */
	typedef TKeyValuePair<FString, FString> StringKeyValue;
	/** When you register a callback for reporting your stats, you will be given a TFunctionRef to call to add stats. This is the signature of the function you will call. */
	typedef TFunctionRef<void(const FString& StatName, const TArray<StringKeyValue>& StatAttributes)> AddStatFuncRef;

	/** To register a callback for reporting stats, create an instance of this delegate type and add it to the delegate variable below. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FGatherCookStatsDelegate, AddStatFuncRef);

	/** Use this to register a callback to gather cook stats for a translation unit. When a cook is finished, this delegate will be fired. */
	static CORE_API FGatherCookStatsDelegate CookStatsCallbacks;

	/** Called after the cook is finished to gather the stats. */
	static CORE_API void LogCookStats(AddStatFuncRef AddStat);

	/** Helper struct to auto-register your STATIC FUNCTION with CookStatsCallbacks */
	struct FAutoRegisterCallback
	{
		template<typename Func>
		explicit FAutoRegisterCallback(Func Callback)
		{
			CookStatsCallbacks.AddLambda(Callback);
		}
	};

	//-------------------------------------------------------------------------
	// Helper to initialize an array of KeyValues on one line.
	//-------------------------------------------------------------------------

	/** Helper to create an array of KeyValues using a single expression. If there is an odd number of arguments, the last value is considered to be an empty string. */
	template <typename...ArgTypes>
	static TArray<FCookStatsManager::StringKeyValue> CreateKeyValueArray(ArgTypes&&...Args)
	{
		TArray<FCookStatsManager::StringKeyValue> Attrs;
		Attrs.Empty(sizeof...(Args) / 2);
		ImplCreateKeyValueArray(Attrs, Forward<ArgTypes>(Args)...);
		return Attrs;
	}

private:
	template <typename KeyType>
	static void ImplCreateKeyValueArray(TArray<FCookStatsManager::StringKeyValue>& Attrs, KeyType&& Key)
	{
		Attrs.Emplace(LexToString(Forward<KeyType>(Key)), TEXT(""));
	}

	template <typename KeyType, typename ValueType>
	static void ImplCreateKeyValueArray(TArray<FCookStatsManager::StringKeyValue>& Attrs, KeyType&& Key, ValueType&& Value)
	{
		Attrs.Emplace(LexToString(Forward<KeyType>(Key)), LexToString(Forward<ValueType>(Value)));
	}

	template <typename KeyType, typename ValueType, typename...ArgTypes>
	static void ImplCreateKeyValueArray(TArray<FCookStatsManager::StringKeyValue>& Attrs, KeyType&& Key, ValueType&& Value, ArgTypes&&...Args)
	{
		Attrs.Emplace(LexToString(Forward<KeyType>(Key)), LexToString(Forward<ValueType>(Value)));
		ImplCreateKeyValueArray(Attrs, Forward<ArgTypes>(Args)...);
	}

};

struct FDerivedDataUsageStats;

/** 
 * Shared code to provide some common cook telemetry functionality.
 * Exposes a CallStats class that exposes counters that track call count, cycles, and "bytes processed" for a call, bucketing by "cache hits/misses" and "game/other thread".
 * This allows us to quickly time a call that may be executing on any thread, but quickly see the main thread time vs other time, split by "hits" or "misses", where hit/miss typically refers to a DDC hit/miss.
 */
class FCookStats
{
public:
	/**
	* Struct to hold stats for a call.
	* The sub-structs and stuff look intimidating, but it is to unify the concept
	* of accumulating any stat on the game or other thread, using raw int64 on the
	* main thread, and thread safe accumulators on other threads.
	*
	* Each call type that will be tracked will track call counts, call cycles, and bytes processed,
	* grouped by hits and misses. Some stats will by definition be zero
	* but it limits the branching in the tracking code.
	*/
	struct CallStats
	{
		/** One group of accumulators for hits and misses. */
		enum class EHitOrMiss : uint8
		{
			Hit,
			Miss,
			MaxValue,
		};

		/** Each hit or miss will contain these accumulator stats. */
		enum class EStatType : uint8
		{
			Counter,
			Cycles,
			Bytes,
			MaxValue,
		};

		/** Contains a pair of accumulators, one for the game thread, one for the other threads. */
		struct GameAndOtherThreadAccumulator
		{
			/** Accumulates a stat. Uses thread safe primitives for non-game thread accumulation. */
			void Accumulate(int64 Value, bool bIsInGameThread)
			{
				if (bIsInGameThread)
				{
					GameThread += Value;
				}
				else if (Value != 0)
				{
					OtherThread.Add(Value);
				}
			}
			/** Access the accumulated values (exposed for more uniform access methds to each accumulator). */
			int64 GetAccumulatedValue(bool bIsInGameThread) const
			{
				return bIsInGameThread ? GameThread : OtherThread.GetValue();
			}

			int64 GameThread = 0l;
			FThreadSafeCounter64 OtherThread;
		};

		/** Make it easier to update an accumulator by providing strongly typed access to the 2D array. */
		void Accumulate(EHitOrMiss HitOrMiss, EStatType StatType, int64 Value, bool bIsInGameThread)
		{
			Accumulators[(uint8)HitOrMiss][(uint8)StatType].Accumulate(Value, bIsInGameThread);
		}

		/** Make it easier to access an accumulator using a uniform, stronly typed interface. */
		int64 GetAccumulatedValue(EHitOrMiss HitOrMiss, EStatType StatType, bool bIsInGameThread) const
		{
			return Accumulators[(uint8)HitOrMiss][(uint8)StatType].GetAccumulatedValue(bIsInGameThread);
		}

		/** Make it easier to access an accumulator using a uniform, stronly typed interface. */
		int64 GetAccumulatedValueAnyThread(EHitOrMiss HitOrMiss, EStatType StatType) const
		{
			return GetAccumulatedValue(HitOrMiss, StatType, true) + GetAccumulatedValue(HitOrMiss, StatType, false);
		}

		/** Used to log the instance in a common way. */
		void LogStats(FCookStatsManager::AddStatFuncRef AddStat, const FString& StatName, const FString& NodeName, const TCHAR* CallName) const
		{
			auto LogStat = [&](EHitOrMiss HitOrMiss, bool bGameThread)
			{
				// only report the stat if it's non-zero
				if (GetAccumulatedValue(HitOrMiss, EStatType::Cycles, bGameThread) > 0)
				{
					AddStat(StatName, FCookStatsManager::CreateKeyValueArray(
						TEXT("ThreadName"), bGameThread ? TEXT("GameThread") : TEXT("OthrThread"),
						TEXT("Call"), CallName,
						TEXT("HitOrMiss"), HitOrMiss == EHitOrMiss::Hit ? TEXT("Hit") : TEXT("Miss"),
						TEXT("Count"), GetAccumulatedValue(HitOrMiss, EStatType::Counter, bGameThread),
						TEXT("TimeSec"), (double)GetAccumulatedValue(HitOrMiss, EStatType::Cycles, bGameThread) * FPlatformTime::GetSecondsPerCycle(),
						TEXT("MB"), (double)GetAccumulatedValue(HitOrMiss, EStatType::Bytes, bGameThread) / (1024.0 * 1024.0),
						TEXT("MB/s"), GetAccumulatedValue(HitOrMiss, EStatType::Cycles, bGameThread) == 0
							? 0.0
							: (double)GetAccumulatedValue(HitOrMiss, EStatType::Bytes, bGameThread) / (1024.0 * 1024.0) /
							 ((double)GetAccumulatedValue(HitOrMiss, EStatType::Cycles, bGameThread) * FPlatformTime::GetSecondsPerCycle()),
						TEXT("Node"), NodeName
						));
				}
			};

			LogStat(EHitOrMiss::Hit, true);
			LogStat(EHitOrMiss::Miss, true);
			LogStat(EHitOrMiss::Hit, false);
			LogStat(EHitOrMiss::Miss, false);
		}

		void Combine(const CallStats& Other)
		{
			for (int32 IsGameThread = 0; IsGameThread <= 1; IsGameThread++)
			{
				for (int32 HitOrMiss = 0; HitOrMiss < (uint8)EHitOrMiss::MaxValue; HitOrMiss++)
				{
					for (int32 StatType = 0; StatType < (uint8)EStatType::MaxValue; StatType++)
					{
						int64 Value = Other.GetAccumulatedValue((EHitOrMiss)HitOrMiss, (EStatType)StatType, IsGameThread == 1);
						Accumulate((EHitOrMiss)HitOrMiss, (EStatType)StatType, Value, IsGameThread == 1);
					}
				}
			}
		}

	private:
		/** The actual accumulators. All access should be from the above public functions. */
		GameAndOtherThreadAccumulator Accumulators[(uint8)EHitOrMiss::MaxValue][(uint8)EStatType::MaxValue];
	};

	/**
	* used to accumulate cycles to a CallStats instance.
	* Will also accumulate hit/miss stats in the dtor as well.
	* If AddHit is not called, it will assume a miss. If AddMiss is called, it will convert the call to a miss.
	*/
	class FScopedStatsCounter
	{
	public:
		/** Starts the time, tracks the underlying stat it will update. Use bCyclesOnly if you need to track additional cycles in a different scope without adding to the call count. */
		explicit FScopedStatsCounter(CallStats& InStats)
			: Stats(InStats)
			, StartTime(FPlatformTime::Cycles64())
			, bIsInGameThread(IsInGameThread())
		{
		}

		FScopedStatsCounter(FScopedStatsCounter&& InOther)
			: Stats(InOther.Stats)
			, StartTime(InOther.StartTime)
			, BytesProcessed(InOther.BytesProcessed)
			, bIsInGameThread(InOther.bIsInGameThread)
			, bCanceled(InOther.bCanceled)
			, bTrackCyclesOnly(InOther.bTrackCyclesOnly)
			, HitOrMiss(InOther.HitOrMiss)
		{
			InOther.Cancel();
		}

		FScopedStatsCounter(const FScopedStatsCounter&) = delete;

		/** Stop the timer and flushes the stats to the underlying stats object. */
		~FScopedStatsCounter()
		{
			if (!bCanceled)
			{
				uint64 CurrentTime = FPlatformTime::Cycles64();
				const uint64 CyclesUsed = CurrentTime - StartTime;

				Stats.Accumulate(HitOrMiss, CallStats::EStatType::Cycles, CyclesUsed, bIsInGameThread);
				if (!bTrackCyclesOnly)
				{
					Stats.Accumulate(HitOrMiss, CallStats::EStatType::Counter, 1l, bIsInGameThread);
				}
				Stats.Accumulate(HitOrMiss, CallStats::EStatType::Bytes, BytesProcessed, bIsInGameThread);
			}
		}

	public:
		/** Call to indicate a Get or Put "cache hit". Exists calls by definition don't have hits or misses. */
		void AddHit(int64 InBytesProcessed)
		{
			HitOrMiss = CallStats::EHitOrMiss::Hit;
			BytesProcessed = InBytesProcessed;
		}

		/** Call to indicate a Get or Put "cache miss". This is optional, as a Miss is assumed by the timer. Some calls by definition don't have hits or misses. */
		void AddMiss(int64 InBytesProcessed = 0)
		{
			HitOrMiss = CallStats::EHitOrMiss::Miss;
			BytesProcessed = InBytesProcessed;
		}

		/** Call to indicate either a hit or miss. See AddHit and AddMiss. */
		void AddHitOrMiss(CallStats::EHitOrMiss InHitOrMiss, int64 InBytesProcessed)
		{
			HitOrMiss = InHitOrMiss;
			BytesProcessed = InBytesProcessed;
		}

		/** Used to cancel a timing, if we had to start one without knowing in advance if we must record it. */
		void Cancel()
		{
			bCanceled = true;
		}

		/** Used to track cycles, but not as a new hit/miss. Used when the timing for a "call" is spread out in the code and mustbe measured in multiple places. */
		void TrackCyclesOnly()
		{
			bTrackCyclesOnly = true;
		}

	private:
		CallStats& Stats;
		uint64 StartTime;
		int64 BytesProcessed = 0;
		bool bIsInGameThread;
		bool bCanceled = false;
		bool bTrackCyclesOnly = false;
		CallStats::EHitOrMiss HitOrMiss = CallStats::EHitOrMiss::Miss;
	};

	using FDDCResourceUsageStats = FDerivedDataUsageStats;
};

/**
 * Stats for a resource class that uses the DDC to load the resource.
 * This supports timing a call for synchronous work and asynchronous waits.
 *
 * SyncWork timing should be any code that is explicitly synchronous, be it a DDC fetch or building the DDC resource.
 * For SyncWork, a "hit" means we fetched from the DDC, and a "miss" means we had to build the resource. Either way,
 * the bytes processed should be set appropriately via calls to AddHitOrMiss().
 *
 * AsyncWait timing should be any code that is waiting on any explicitly asynchronous work to complete.
 * This should only time the wait portion.
 * A "hit" or "miss" should be interpreted the same way as in SyncWork.
 * The Cycles counter should be measuring WAIT time, NOT the time the async task took to complete.
 *
 * Many systems build assets in a very different spot in the code than they are retrieved. When that happens
 * TrackCyclesOnly() should be called on the timer, and the Miss should be timed where the resource
 * is actually built so the size can cycles used to build the resource can be tracked properly.
 * In such cases, the time to TRY to get from the DDC will be timed along with the actual time to build the resource
 * But there will only be a single CallCount recorded. This is desired so that every Miss() counted will properly equate
 * to one resource built. Look for calls to TrackCyclesOnly() to see how this works in practice.
 *
 * Async tasks can sometimes actually be executed synchronously if the code ends up waiting on the results
 * before the task can even get started. Either way, this should be treated as "async" wait time. Be sure
 * not to double count this time. Basically, do your counting at the level that decides whether to execute
 * the work synchronously or not, so you can be sure to track the time properly as SyncWork or AsyncWait.
 */
struct FDerivedDataUsageStats
{
	/** Call this where the code is building the resource synchronously */
	FCookStats::FScopedStatsCounter TimeSyncWork()
	{
		return FCookStats::FScopedStatsCounter(SyncWorkStats);
	}

	/** Call this where the code is waiting on async work to build the resource */
	FCookStats::FScopedStatsCounter TimeAsyncWait()
	{
		return FCookStats::FScopedStatsCounter(AsyncWaitStats);
	}

	// expose these publicly for low level access. These should really never be accessed directly except when finished accumulating them.
	FCookStats::CallStats SyncWorkStats;
	FCookStats::CallStats AsyncWaitStats;

	void LogStats(FCookStatsManager::AddStatFuncRef AddStat, const FString& StatName, const FString& NodeName) const
	{
		SyncWorkStats.LogStats(AddStat, StatName, NodeName, TEXT("SyncWork"));
		AsyncWaitStats.LogStats(AddStat, StatName, NodeName, TEXT("AsyncWait"));
	}
};

/** any expression inside this block will be compiled out if ENABLE_COOK_STATS is not true. Shorthand for the multi-line #if ENABLE_COOK_STATS guard. */
#define COOK_STAT(...) __VA_ARGS__

UE_TRACE_CHANNEL_EXTERN(CookChannel, CORE_API);

#define UE_INSIGHTS_COOK_PROFILER_VERSION 2

CORE_API void TracePackage(uint64 InId, const FStringView InName);
CORE_API void TracePackageAssetClass(uint64 InId, const FStringView InName);
CORE_API bool ShouldTracePackageInfo();

#if UE_INSIGHTS_COOK_PROFILER_VERSION == 2

CORE_API void TracePackageStatBeginScope(uint64 InId, uint64 Time, EPackageEventStatType StatType);
CORE_API void TracePackageStatEndScope(uint64 InId, uint64 Time, EPackageEventStatType StatType);

struct FScopedCookStat
{
	FScopedCookStat(const FScopedCookStat&) = delete;
	FScopedCookStat(FScopedCookStat&&) = delete;

	FScopedCookStat(uint64 Id, EPackageEventStatType InStatType)
		: Id(Id)
		, StatType(InStatType)
	{
		TracePackageStatBeginScope(Id, FPlatformTime::Cycles64(), StatType);
	}

	~FScopedCookStat()
	{
		TracePackageStatEndScope(Id, FPlatformTime::Cycles64(), StatType);
	}

private:
	uint64 Id;
	EPackageEventStatType StatType;
};

#endif

#if UE_INSIGHTS_COOK_PROFILER_VERSION == 1

struct FScopedCookStat
{
	FScopedCookStat(const FScopedCookStat&) = delete;
	FScopedCookStat(FScopedCookStat&&) = delete;

	FScopedCookStat(uint64 Id, EPackageEventStatType InStatType)
		: Id(Id)
		, StatType(InStatType)
	{
		StartTimestamp = FPlatformTime::Cycles64();
	}

	~FScopedCookStat()
	{
		uint64 Duration = FPlatformTime::Cycles64() - StartTimestamp;
		TracePackageStat(Id, Duration, StatType);
	}

private:
	uint64 Id;
	uint64 StartTimestamp;
	EPackageEventStatType StatType;
};

#endif

#define UE_SCOPED_COOK_STAT(Name, StatType) \
	FScopedCookStat  UE_JOIN(__CookTimerScope, __LINE__)(Name.ToUnstableInt(), StatType);

#define UE_MULTI_SCOPED_COOK_STAT_INIT() \
	TOptional<FScopedCookStat> __CookStatScope; \
	FName __PrevPackageName = NAME_None;

#define UE_MULTI_SCOPED_COOK_STAT(Name, StatType) \
	if (__PrevPackageName != Name) \
	{ \
		__CookStatScope.Emplace(Name.ToUnstableInt(), EPackageEventStatType::LoadPackage); \
		__PrevPackageName = Name; \
	}

#define UE_MULTI_SCOPED_COOK_STAT_RESET() \
	__CookStatScope.Reset(); \
	__PrevPackageName = NAME_None;
#else

#define COOK_STAT(...)
#define UE_SCOPED_COOK_STAT(...)
#define UE_MULTI_SCOPED_COOK_STAT_INIT(...)
#define UE_MULTI_SCOPED_COOK_STAT(...)
#define UE_MULTI_SCOPED_COOK_STAT_RESET(...)

#define TracePackage(...)
#define TracePackageStat(...)
#define TracePackageStatBeginScope(...)
#define TracePackageStatEndScope(...)
#define TracePackageAssetClass(...)
#define ShouldTracePackageInfo(...) false

#endif
