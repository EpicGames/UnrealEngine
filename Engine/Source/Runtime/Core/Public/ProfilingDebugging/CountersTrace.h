// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PreprocessorHelpers.h"
#include "Misc/Build.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Trace/Config.h"
#include "Trace/Detail/Channel.h"
#include "Trace/Detail/Channel.inl"
#include "Trace/Trace.h"

#include <atomic>

#if !defined(COUNTERSTRACE_ENABLED)
#if UE_TRACE_ENABLED && !UE_BUILD_SHIPPING
#define COUNTERSTRACE_ENABLED 1
#else
#define COUNTERSTRACE_ENABLED 0
#endif
#endif

enum ETraceCounterType : uint8
{
	TraceCounterType_Int = 0,
	TraceCounterType_Float = 1,
};

enum ETraceCounterDisplayHint : uint8
{
	TraceCounterDisplayHint_None = 0,
	TraceCounterDisplayHint_Memory = 1,
};

enum ETraceCounterNameType : uint8
{
	TraceCounterNameType_Static = 0, // TCounter is allowed to keep a pointer to InCounterName string
	TraceCounterNameType_Dynamic = 0x10, // TCounter needs to copy the InCounterName string
	TraceCounterNameType_AllocNameCopy = 0x20, // TCounter has allocated a copy of the InCounterName string
};

#if COUNTERSTRACE_ENABLED

UE_TRACE_CHANNEL_EXTERN(CountersChannel, CORE_API);

struct FCountersTrace
{
	CORE_API static uint16 OutputInitCounter(const TCHAR* CounterName, ETraceCounterType CounterType, ETraceCounterDisplayHint CounterDisplayHint);
	CORE_API static void OutputSetValue(uint16 CounterId, int64 Value);
	CORE_API static void OutputSetValue(uint16 CounterId, double Value);

	CORE_API static const TCHAR* AllocAndCopyCounterName(const TCHAR* InCounterName);
	CORE_API static void FreeCounterName(const TCHAR* InCounterName);

	template<typename ValueType, ETraceCounterType CounterType, typename StoredType = ValueType, bool bUnchecked = false>
	class TCounter
	{
	public:
		TCounter() = delete;

		template<int N>
		TCounter(const TCHAR(&InCounterName)[N], ETraceCounterDisplayHint InCounterDisplayHint)
			: Value(0)
			, CounterName(InCounterName) // assumes that InCounterName is a static string
			, CounterId(0)
			, CounterFlags(uint8(InCounterDisplayHint))
		{
			CounterId = OutputInitCounter(InCounterName, CounterType, InCounterDisplayHint);
		}

		TCounter(ETraceCounterNameType InCounterNameType, const TCHAR* InCounterName, ETraceCounterDisplayHint InCounterDisplayHint)
			: Value(0)
			, CounterName(InCounterName)
			, CounterId(0)
			, CounterFlags(uint8(InCounterDisplayHint))
		{
			CounterId = OutputInitCounter(InCounterName, CounterType, InCounterDisplayHint);

			if (CounterId == 0 && InCounterNameType == TraceCounterNameType_Dynamic)
			{
				// Store counter name for late init. Needs a copy as InCounterName pointer might not be valid later.
				CounterName = AllocAndCopyCounterName(InCounterName);
				CounterFlags |= uint8(TraceCounterNameType_AllocNameCopy);
			}
		}

		~TCounter()
		{
			if (CounterFlags & uint8(TraceCounterNameType_AllocNameCopy))
			{
				FreeCounterName(CounterName);
			}
		}

		void LateInit()
		{
			uint32 OldId = CounterId.load();
			if (!OldId)
			{
				uint32 NewId = OutputInitCounter(CounterName, CounterType, ETraceCounterDisplayHint(CounterFlags & 0xF));
				CounterId.compare_exchange_weak(OldId, NewId);
			}
		}

		ValueType Get() const
		{
			return Value;
		}

		void Set(ValueType InValue)
		{
			if (bUnchecked || Value != InValue)
			{
				Value = InValue;
				if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CountersChannel))
				{
					LateInit();
					OutputSetValue(uint16(CounterId), Value);
				}
			}
		}

		void SetIfDifferent(ValueType InValue)
		{
			if (Value != InValue)
			{
				Value = InValue;
				if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CountersChannel))
				{
					LateInit();
					OutputSetValue(uint16(CounterId), Value);
				}
			}
		}

		void SetAlways(ValueType InValue)
		{
			Value = InValue;
			if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CountersChannel))
			{
				LateInit();
				OutputSetValue(uint16(CounterId), Value);
			}
		}

		void Add(ValueType InValue)
		{
			if (bUnchecked || InValue != 0)
			{
				Value += InValue;
				if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CountersChannel))
				{
					LateInit();
					OutputSetValue(uint16(CounterId), Value);
				}
			}
		}

		void AddIfNotZero(ValueType InValue)
		{
			if (InValue != 0)
			{
				Value += InValue;
				if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CountersChannel))
				{
					LateInit();
					OutputSetValue(uint16(CounterId), Value);
				}
			}
		}

		void AddAlways(ValueType InValue)
		{
			Value += InValue;
			if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CountersChannel))
			{
				LateInit();
				OutputSetValue(uint16(CounterId), Value);
			}
		}

		void Subtract(ValueType InValue)
		{
			if (bUnchecked || InValue != 0)
			{
				Value -= InValue;
				if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CountersChannel))
				{
					LateInit();
					OutputSetValue(uint16(CounterId), Value);
				}
			}
		}

		void SubtractIfNotZero(ValueType InValue)
		{
			if (InValue != 0)
			{
				Value -= InValue;
				if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CountersChannel))
				{
					LateInit();
					OutputSetValue(uint16(CounterId), Value);
				}
			}
		}

		void SubtractAlways(ValueType InValue)
		{
			Value -= InValue;
			if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CountersChannel))
			{
				LateInit();
				OutputSetValue(uint16(CounterId), Value);
			}
		}

		void Increment()
		{
			++Value;
			if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CountersChannel))
			{
				LateInit();
				OutputSetValue(uint16(CounterId), Value);
			}
		}

		void Decrement()
		{
			--Value;
			if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CountersChannel))
			{
				LateInit();
				OutputSetValue(uint16(CounterId), Value);
			}
		}

	private:
		StoredType Value;
		const TCHAR* CounterName;
		std::atomic<uint32> CounterId;
		uint8 CounterFlags; // stores a combination of ETraceCounterDisplayHint and ETraceCounterNameType flags
	};

	using FCounterInt = TCounter<int64, TraceCounterType_Int>;
	using FCounterAtomicInt = TCounter<int64, TraceCounterType_Int, std::atomic<int64>>;
	using FCounterFloat = TCounter<double, TraceCounterType_Float>;
	using FCounterAtomicFloat = TCounter<double, TraceCounterType_Float, std::atomic<double>>;

	using FCounterUncheckedInt = TCounter<int64, TraceCounterType_Int, int64, true>;
	using FCounterUncheckedAtomicInt = TCounter<int64, TraceCounterType_Int, std::atomic<int64>, true>;
	using FCounterUncheckedFloat = TCounter<double, TraceCounterType_Float, double, true>;
	using FCounterUncheckedAtomicFloat = TCounter<double, TraceCounterType_Float, std::atomic<double>, true>;
};

//////////////////////////////////////////////////
// Inline Counters

#define __TRACE_CHECK_COUNTER_NAME(CounterDisplayName) \
	static_assert(std::is_const_v<std::remove_reference_t<decltype(CounterDisplayName)>>, "CounterDisplayName string must be a const TCHAR array."); \
	static_assert(TIsArrayOrRefOfTypeByPredicate<decltype(CounterDisplayName), TIsCharEncodingCompatibleWithTCHAR>::Value, "CounterDisplayName string must be a TCHAR array.");

#define __TRACE_DECLARE_INLINE_COUNTER(CounterDisplayName, CounterType, CounterDisplayHint) \
	__TRACE_CHECK_COUNTER_NAME(CounterDisplayName) \
	static FCountersTrace::CounterType UE_JOIN(__TraceCounter, __LINE__)(CounterDisplayName, CounterDisplayHint);

#define __TRACE_INLINE_COUNTER_SET(CounterDisplayName, Value, CounterType, CounterDisplayHint) \
	__TRACE_DECLARE_INLINE_COUNTER(CounterDisplayName, CounterType, CounterDisplayHint) \
	UE_JOIN(__TraceCounter, __LINE__).Set(Value);

#define TRACE_INT_VALUE(CounterDisplayName, Value)                      __TRACE_INLINE_COUNTER_SET(CounterDisplayName, Value, FCounterInt, TraceCounterDisplayHint_None)
#define TRACE_ATOMIC_INT_VALUE(CounterDisplayName, Value)               __TRACE_INLINE_COUNTER_SET(CounterDisplayName, Value, FCounterAtomicInt, TraceCounterDisplayHint_None)
#define TRACE_FLOAT_VALUE(CounterDisplayName, Value)                    __TRACE_INLINE_COUNTER_SET(CounterDisplayName, Value, FCounterFloat, TraceCounterDisplayHint_None)
#define TRACE_ATOMIC_FLOAT_VALUE(CounterDisplayName, Value)             __TRACE_INLINE_COUNTER_SET(CounterDisplayName, Value, FCounterAtomicFloat, TraceCounterDisplayHint_None)
#define TRACE_MEMORY_VALUE(CounterDisplayName, Value)                   __TRACE_INLINE_COUNTER_SET(CounterDisplayName, Value, FCounterInt, TraceCounterDisplayHint_Memory)
#define TRACE_ATOMIC_MEMORY_VALUE(CounterDisplayName, Value)            __TRACE_INLINE_COUNTER_SET(CounterDisplayName, Value, FCounterAtomicInt, TraceCounterDisplayHint_Memory)

#define TRACE_UNCHECKED_INT_VALUE(CounterDisplayName, Value)            __TRACE_INLINE_COUNTER_SET(CounterDisplayName, Value, FCounterUncheckedInt, TraceCounterDisplayHint_None)
#define TRACE_UNCHECKED_ATOMIC_INT_VALUE(CounterDisplayName, Value)     __TRACE_INLINE_COUNTER_SET(CounterDisplayName, Value, FCounterUncheckedAtomicInt, TraceCounterDisplayHint_None)
#define TRACE_UNCHECKED_FLOAT_VALUE(CounterDisplayName, Value)          __TRACE_INLINE_COUNTER_SET(CounterDisplayName, Value, FCounterUncheckedFloat, TraceCounterDisplayHint_None)
#define TRACE_UNCHECKED_ATOMIC_FLOAT_VALUE(CounterDisplayName, Value)   __TRACE_INLINE_COUNTER_SET(CounterDisplayName, Value, FCounterUncheckedAtomicFloat, TraceCounterDisplayHint_None)
#define TRACE_UNCHECKED_MEMORY_VALUE(CounterDisplayName, Value)         __TRACE_INLINE_COUNTER_SET(CounterDisplayName, Value, FCounterUncheckedInt, TraceCounterDisplayHint_Memory)
#define TRACE_UNCHECKED_ATOMIC_MEMORY_VALUE(CounterDisplayName, Value)  __TRACE_INLINE_COUNTER_SET(CounterDisplayName, Value, FCounterUncheckedAtomicInt, TraceCounterDisplayHint_Memory)

//////////////////////////////////////////////////

#define TRACE_DECLARE_COUNTER(CounterType, CounterName, CounterDisplayName, CounterDisplayHint) \
	__TRACE_CHECK_COUNTER_NAME(CounterDisplayName) \
	FCountersTrace::CounterType UE_JOIN(__GTraceCounter, CounterName)(CounterDisplayName, CounterDisplayHint);

//////////////////////////////////////////////////
// Declare Int Counters

#define TRACE_DECLARE_INT_COUNTER(CounterName, CounterDisplayName) \
	TRACE_DECLARE_COUNTER(FCounterInt, CounterName, CounterDisplayName, TraceCounterDisplayHint_None)

#define TRACE_DECLARE_UNCHECKED_INT_COUNTER(CounterName, CounterDisplayName) \
	TRACE_DECLARE_COUNTER(FCounterUncheckedInt, CounterName, CounterDisplayName, TraceCounterDisplayHint_None)

#define TRACE_DECLARE_INT_COUNTER_EXTERN(CounterName) \
	extern FCountersTrace::FCounterInt UE_JOIN(__GTraceCounter, CounterName);

#define TRACE_DECLARE_UNCHECKED_INT_COUNTER_EXTERN(CounterName) \
	extern FCountersTrace::FCounterUncheckedInt UE_JOIN(__GTraceCounter, CounterName);

//////////////////////////////////////////////////
// Declare Atomic Int Counters

#define TRACE_DECLARE_ATOMIC_INT_COUNTER(CounterName, CounterDisplayName) \
	TRACE_DECLARE_COUNTER(FCounterAtomicInt, CounterName, CounterDisplayName, TraceCounterDisplayHint_None)

#define TRACE_DECLARE_UNCHECKED_ATOMIC_INT_COUNTER(CounterName, CounterDisplayName) \
	TRACE_DECLARE_COUNTER(FCounterUncheckedAtomicInt, CounterName, CounterDisplayName, TraceCounterDisplayHint_None)

#define TRACE_DECLARE_ATOMIC_INT_COUNTER_EXTERN(CounterName) \
	extern FCountersTrace::FCounterAtomicInt UE_JOIN(__GTraceCounter, CounterName);

#define TRACE_DECLARE_UNCHECKED_ATOMIC_INT_COUNTER_EXTERN(CounterName) \
	extern FCountersTrace::FCounterUncheckedAtomicInt UE_JOIN(__GTraceCounter, CounterName);

//////////////////////////////////////////////////
// Declare Float Counters

#define TRACE_DECLARE_FLOAT_COUNTER(CounterName, CounterDisplayName) \
	TRACE_DECLARE_COUNTER(FCounterFloat, CounterName, CounterDisplayName, TraceCounterDisplayHint_None)

#define TRACE_DECLARE_UNCHECKED_FLOAT_COUNTER(CounterName, CounterDisplayName) \
	TRACE_DECLARE_COUNTER(FCounterUncheckedFloat, CounterName, CounterDisplayName, TraceCounterDisplayHint_None)

#define TRACE_DECLARE_FLOAT_COUNTER_EXTERN(CounterName) \
	extern FCountersTrace::FCounterFloat UE_JOIN(__GTraceCounter, CounterName);

#define TRACE_DECLARE_UNCHECKED_FLOAT_COUNTER_EXTERN(CounterName) \
	extern FCountersTrace::FCounterUncheckedFloat UE_JOIN(__GTraceCounter, CounterName);

//////////////////////////////////////////////////
// Declare Atomic Float Counters

#define TRACE_DECLARE_ATOMIC_FLOAT_COUNTER(CounterName, CounterDisplayName) \
	TRACE_DECLARE_COUNTER(FCounterAtomicFloat, CounterName, CounterDisplayName, TraceCounterDisplayHint_None)

#define TRACE_DECLARE_UNCHECKED_ATOMIC_FLOAT_COUNTER(CounterName, CounterDisplayName) \
	TRACE_DECLARE_COUNTER(FCounterUncheckedAtomicFloat, CounterName, CounterDisplayName, TraceCounterDisplayHint_None)

#define TRACE_DECLARE_ATOMIC_FLOAT_COUNTER_EXTERN(CounterName) \
	extern FCountersTrace::FCounterAtomicFloat UE_JOIN(__GTraceCounter, CounterName);

#define TRACE_DECLARE_UNCHECKED_ATOMIC_FLOAT_COUNTER_EXTERN(CounterName) \
	extern FCountersTrace::FCounterUncheckedAtomicFloat UE_JOIN(__GTraceCounter, CounterName);

//////////////////////////////////////////////////
// Declare Memory Counters

#define TRACE_DECLARE_MEMORY_COUNTER(CounterName, CounterDisplayName) \
	TRACE_DECLARE_COUNTER(FCounterInt, CounterName, CounterDisplayName, TraceCounterDisplayHint_Memory)

#define TRACE_DECLARE_UNCHECKED_MEMORY_COUNTER(CounterName, CounterDisplayName) \
	TRACE_DECLARE_COUNTER(FCounterUncheckedInt, CounterName, CounterDisplayName, TraceCounterDisplayHint_Memory)

#define TRACE_DECLARE_MEMORY_COUNTER_EXTERN(CounterName) \
	TRACE_DECLARE_INT_COUNTER_EXTERN(CounterName)

#define TRACE_DECLARE_UNCHECKED_MEMORY_COUNTER_EXTERN(CounterName) \
	TRACE_DECLARE_UNCHECKED_INT_COUNTER_EXTERN(CounterName)

//////////////////////////////////////////////////
// Declare Atomic Memory Counters

#define TRACE_DECLARE_ATOMIC_MEMORY_COUNTER(CounterName, CounterDisplayName) \
	TRACE_DECLARE_COUNTER(FCounterAtomicInt, CounterName, CounterDisplayName, TraceCounterDisplayHint_Memory)

#define TRACE_DECLARE_UNCHECKED_ATOMIC_MEMORY_COUNTER(CounterName, CounterDisplayName) \
	TRACE_DECLARE_COUNTER(FCounterUncheckedAtomicInt, CounterName, CounterDisplayName, TraceCounterDisplayHint_Memory)

#define TRACE_DECLARE_ATOMIC_MEMORY_COUNTER_EXTERN(CounterName) \
	TRACE_DECLARE_ATOMIC_INT_COUNTER_EXTERN(CounterName)

#define TRACE_DECLARE_UNCHECKED_ATOMIC_MEMORY_COUNTER_EXTERN(CounterName) \
	TRACE_DECLARE_UNCHECKED_ATOMIC_INT_COUNTER_EXTERN(CounterName)

//////////////////////////////////////////////////
// Counter Operations

// A value that does not change will be traced (or not) depending on how the counter is created.

#define TRACE_COUNTER_SET(CounterName, Value) \
	UE_JOIN(__GTraceCounter, CounterName).Set(Value);

#define TRACE_COUNTER_GET(CounterName) \
	UE_JOIN(__GTraceCounter, CounterName).Get();

#define TRACE_COUNTER_ADD(CounterName, Value) \
	UE_JOIN(__GTraceCounter, CounterName).Add(Value);

#define TRACE_COUNTER_SUBTRACT(CounterName, Value) \
	UE_JOIN(__GTraceCounter, CounterName).Subtract(Value);

#define TRACE_COUNTER_INCREMENT(CounterName) \
	UE_JOIN(__GTraceCounter, CounterName).Increment();

#define TRACE_COUNTER_DECREMENT(CounterName) \
	UE_JOIN(__GTraceCounter, CounterName).Decrement();

// _IF_DIFFERENT / _IF_NOT_ZERO
// It will not trace a value that doesn't change (no matter how the counter was created).

#define TRACE_COUNTER_SET_IF_DIFFERENT(CounterName, Value) \
	UE_JOIN(__GTraceCounter, CounterName).SetIfDifferent(Value);

#define TRACE_COUNTER_ADD_IF_NOT_ZERO(CounterName, Value) \
	UE_JOIN(__GTraceCounter, CounterName).AddIfNotZero(Value);

#define TRACE_COUNTER_SUBTRACT_IF_NOT_ZERO(CounterName, Value) \
	UE_JOIN(__GTraceCounter, CounterName).SubtractIfNotZero(Value);

// _ALWAYS
// It will trace even if value doesn't change (no matter how the counter was created).

#define TRACE_COUNTER_SET_ALWAYS(CounterName, Value) \
	UE_JOIN(__GTraceCounter, CounterName).SetAlways(Value);

#define TRACE_COUNTER_ADD_ALWAYS(CounterName, Value) \
	UE_JOIN(__GTraceCounter, CounterName).AddAlways(Value);

#define TRACE_COUNTER_SUBTRACT_ALWAYS(CounterName, Value) \
	UE_JOIN(__GTraceCounter, CounterName).SubtractAlways(Value);

#else // COUNTERSTRACE_ENABLED

#define TRACE_INT_VALUE(CounterDisplayName, Value)
#define TRACE_ATOMIC_INT_VALUE(CounterDisplayName, Value)
#define TRACE_FLOAT_VALUE(CounterDisplayName, Value)
#define TRACE_ATOMIC_FLOAT_VALUE(CounterDisplayName, Value)
#define TRACE_MEMORY_VALUE(CounterDisplayName, Value)
#define TRACE_ATOMIC_MEMORY_VALUE(CounterDisplayName, Value)

#define TRACE_UNCHECKED_INT_VALUE(CounterDisplayName, Value)
#define TRACE_UNCHECKED_ATOMIC_INT_VALUE(CounterDisplayName, Value)
#define TRACE_UNCHECKED_FLOAT_VALUE(CounterDisplayName, Value)
#define TRACE_UNCHECKED_ATOMIC_FLOAT_VALUE(CounterDisplayName, Value)
#define TRACE_UNCHECKED_MEMORY_VALUE(CounterDisplayName, Value)
#define TRACE_UNCHECKED_ATOMIC_MEMORY_VALUE(CounterDisplayName, Value)

#define TRACE_DECLARE_COUNTER(CounterType, CounterName, CounterDisplayName, CounterDisplayHint)

#define TRACE_DECLARE_INT_COUNTER(CounterName, CounterDisplayName)
#define TRACE_DECLARE_INT_COUNTER_EXTERN(CounterName)
#define TRACE_DECLARE_ATOMIC_INT_COUNTER(CounterName, CounterDisplayName)
#define TRACE_DECLARE_ATOMIC_INT_COUNTER_EXTERN(CounterName)
#define TRACE_DECLARE_FLOAT_COUNTER(CounterName, CounterDisplayName)
#define TRACE_DECLARE_FLOAT_COUNTER_EXTERN(CounterName)
#define TRACE_DECLARE_ATOMIC_FLOAT_COUNTER(CounterName, CounterDisplayName)
#define TRACE_DECLARE_ATOMIC_FLOAT_COUNTER_EXTERN(CounterName)
#define TRACE_DECLARE_MEMORY_COUNTER(CounterName, CounterDisplayName)
#define TRACE_DECLARE_MEMORY_COUNTER_EXTERN(CounterName)
#define TRACE_DECLARE_ATOMIC_MEMORY_COUNTER(CounterName, CounterDisplayName)
#define TRACE_DECLARE_ATOMIC_MEMORY_COUNTER_EXTERN(CounterName)

#define TRACE_DECLARE_UNCHECKED_INT_COUNTER(CounterName, CounterDisplayName)
#define TRACE_DECLARE_UNCHECKED_INT_COUNTER_EXTERN(CounterName)
#define TRACE_DECLARE_UNCHECKED_ATOMIC_INT_COUNTER(CounterName, CounterDisplayName)
#define TRACE_DECLARE_UNCHECKED_ATOMIC_INT_COUNTER_EXTERN(CounterName)
#define TRACE_DECLARE_UNCHECKED_FLOAT_COUNTER(CounterName, CounterDisplayName)
#define TRACE_DECLARE_UNCHECKED_FLOAT_COUNTER_EXTERN(CounterName)
#define TRACE_DECLARE_UNCHECKED_ATOMIC_FLOAT_COUNTER(CounterName, CounterDisplayName)
#define TRACE_DECLARE_UNCHECKED_ATOMIC_FLOAT_COUNTER_EXTERN(CounterName)
#define TRACE_DECLARE_UNCHECKED_MEMORY_COUNTER(CounterName, CounterDisplayName)
#define TRACE_DECLARE_UNCHECKED_MEMORY_COUNTER_EXTERN(CounterName)
#define TRACE_DECLARE_UNCHECKED_ATOMIC_MEMORY_COUNTER(CounterName, CounterDisplayName)
#define TRACE_DECLARE_UNCHECKED_ATOMIC_MEMORY_COUNTER_EXTERN(CounterName)

#define TRACE_COUNTER_SET(CounterName, Value)
#define TRACE_COUNTER_GET(CounterName) {}
#define TRACE_COUNTER_ADD(CounterName, Value)
#define TRACE_COUNTER_SUBTRACT(CounterName, Value)
#define TRACE_COUNTER_INCREMENT(CounterName)
#define TRACE_COUNTER_DECREMENT(CounterName)

#define TRACE_COUNTER_SET_IF_DIFFERENT(CounterName, Value)
#define TRACE_COUNTER_ADD_IF_NOT_ZERO(CounterName, Value)
#define TRACE_COUNTER_SUBTRACT_IF_NOT_ZERO(CounterName, Value)

#define TRACE_COUNTER_SET_ALWAYS(CounterName, Value)
#define TRACE_COUNTER_ADD_ALWAYS(CounterName, Value)
#define TRACE_COUNTER_SUBTRACT_ALWAYS(CounterName, Value)

#endif // COUNTERSTRACE_ENABLED
