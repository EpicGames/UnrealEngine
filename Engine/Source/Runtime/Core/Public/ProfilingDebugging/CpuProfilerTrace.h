// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_exports

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PreprocessorHelpers.h"
#include "Misc/Build.h"
#include "Trace/Config.h"
#include "Trace/Detail/Channel.h"
#include "Trace/Detail/Channel.inl"
#include "Trace/Trace.h"

// IWYU pragma: end_exports

#if !defined(CPUPROFILERTRACE_ENABLED)
#if UE_TRACE_ENABLED && !UE_BUILD_SHIPPING
#define CPUPROFILERTRACE_ENABLED 1
#else
#define CPUPROFILERTRACE_ENABLED 0
#endif
#endif

#if CPUPROFILERTRACE_ENABLED

UE_TRACE_CHANNEL_EXTERN(CpuChannel, CORE_API);

class FName;

/*
 * Facilities for tracing timed CPU events. Two types of events are supported, static/constant events where the identifier is
 * known at compile time or it is constant during execution, and dynamic events were identifier can change between calls.
 * Static/constant events have lower overhead, so always prefer to use them if possible.
 *
 * Events are tracked per thread, so begin/end calls must be matched and called on the same thread. It is possible to use any channel
 * to emit the events, but both that channel and the CpuChannel must then be enabled.
 *
 * Usage of the scope macros is highly encouraged in order to avoid mistakes.
 */
struct FCpuProfilerTrace
{
	typedef const FName& FNameParam;

	/*
	 * Output CPU event definition (spec).
	 * The trace event emitted by this function is an "important event" (so all events emitted will add to the Trace System's cache).
	 * It is the responsibility of the caller code to ensure this function is not abused.
	 * @param Name - The event name
	 * @param File - The source filename
	 * @param Line - The line number in source file
	 * @returns The event definition id
	 */
	FORCENOINLINE CORE_API static uint32 OutputEventType(const ANSICHAR* Name, const ANSICHAR* File = nullptr, uint32 Line = 0);
	FORCENOINLINE CORE_API static uint32 OutputEventType(const TCHAR* Name, const ANSICHAR* File = nullptr, uint32 Line = 0);
	FORCENOINLINE CORE_API static uint32 OutputEventType(FNameParam Name, const ANSICHAR* File = nullptr, uint32 Line = 0);

	/*
	 * Output CPU event metadata spec.
	 * The trace event emitted by this function is an "important event" (so all events emitted will add to the Trace System's cache).
	 * It is the responsibility of the caller code to ensure this function is not abused.
	 * @param SpecId - The SpecId of the event to associate this metadata spec with
	 * @param StaticName - The static name of the event
	 * @param NameFormat - The format string for the event
	 * @param FieldNames - Cbor data representing the serialized field names. These should be descriptive names for the metadata values passed to OutputBeginEventWithMetadata.
	 */
	FORCENOINLINE CORE_API static void OutputEventMetadataSpec(uint32 SpecId, const TCHAR* StaticName, const TCHAR* NameFormat, const TArray<uint8>& FieldNames);

	/*
	 * Output CPU event metadata. This metadata can than be associated with multiple timing events emitted with OutputBeginEventWithMetadata.
	 * @param SpecId - The SpecId of the event to associate this metadata with.
	 * @param CborData - The serialized metadata associated with the event.
	 * @returns A MetadataId that can be used with OutputBeginEventWithMetadata. Returns 0 if the cpu channel is off.
	 */
	FORCENOINLINE CORE_API static uint32 OutputMetadata(uint32 SpecId, const TArray<uint8>& CborData);

	/*
	 * Output CPU event definition (spec) for a dynamic event.
	 * The Name will be cached and the trace event will only be emitted once (for each unique Name; even if File or Line changes).
	 * @param Name - The event name
	 * @param File - The source filename
	 * @param Line - The line number in source file
	 * @returns The event definition id
	 */
	FORCENOINLINE CORE_API static uint32 OutputDynamicEventType(const ANSICHAR* Name, const ANSICHAR* File = nullptr, uint32 Line = 0);
	FORCENOINLINE CORE_API static uint32 OutputDynamicEventType(const TCHAR* Name, const ANSICHAR* File = nullptr, uint32 Line = 0);
	FORCENOINLINE CORE_API static uint32 OutputDynamicEventType(FNameParam Name, const ANSICHAR* File = nullptr, uint32 Line = 0);

	/*
	 * Output CPU event definition (spec) for a dynamic event identified by an FName.
	 * The Id will be cached and the trace event will only be emitted once (for each unique Id; even if Name, File or Line changes).
	 * This is faster and less memory expensive than \ref OutputDynamicEventType that receives ANSICHAR* or TCHAR* name.
	 * @param Id - The id of event
	 * @param Name - The name of event
	 * @param File - The source filename
	 * @param Line - The line number in source file
	 */
	FORCENOINLINE CORE_API static uint32 OutputDynamicEventTypeWithId(FNameParam Id, const ANSICHAR* Name, const ANSICHAR* File = nullptr, uint32 Line = 0);
	FORCENOINLINE CORE_API static uint32 OutputDynamicEventTypeWithId(FNameParam Id, const TCHAR* Name, const ANSICHAR* File = nullptr, uint32 Line = 0);

	/*
	 * Output begin event marker for a given spec. Must always be matched with an end event.
	 * @param SpecId - The event definition id.
	 */
	CORE_API static void OutputBeginEvent(uint32 SpecId);

	/*
	 * Output begin event marker with metadata for a given spec. Must always be matched with an end event.
	 * @param MetadataId - The MetadataId returned from OutputMetadata.
	 */
	CORE_API static void OutputBeginEventWithMetadata(uint32 MetadataId);

	/*
	 * Output end event marker for an event traced with OutputBeginEventWithMetadata.
	 */
	CORE_API static void OutputEndEventWithMetadata();

	/*
	 * Output begin event marker for a dynamic event name. This is more expensive than statically known event
	 * names using \ref OutputBeginEvent. Must always be matched with an end event.
	 * @param Name - The name of event
	 * @param File - The source filename
	 * @param Line - The line number in source file
	 */
	CORE_API static void OutputBeginDynamicEvent(const ANSICHAR* Name, const ANSICHAR* File = nullptr, uint32 Line = 0);
	CORE_API static void OutputBeginDynamicEvent(const TCHAR* Name, const ANSICHAR* File = nullptr, uint32 Line = 0);

	/*
	 * Output begin event marker for a dynamic event identified by an FName. This is more expensive than
	 * statically known event names using \ref OutputBeginEvent, but it is faster than \ref OutputBeginDynamicEvent
	 * that receives ANSICHAR* / TCHAR* name. Must always be matched with an end event.
	 * @param Name - The name of event
	 * @param File - The source filename
	 * @param Line - The line number in source file
	 */
	CORE_API static void OutputBeginDynamicEvent(FNameParam Name, const ANSICHAR* File = nullptr, uint32 Line = 0);

	/*
	 * Output begin event marker for a dynamic event identified by an FName. This is more expensive than
	 * statically known event names using \ref OutputBeginEvent, but it is faster than \ref OutputBeginDynamicEvent
	 * that receives ANSICHAR* / TCHAR* name. Must always be matched with an end event.
	 * @param Id - The id of event
	 * @param Name - The name of event
	 * @param File - The source filename
	 * @param Line - The line number in source file
	 */
	CORE_API static void OutputBeginDynamicEventWithId(FNameParam Id, const ANSICHAR* Name, const ANSICHAR* File = nullptr, uint32 Line = 0);
	CORE_API static void OutputBeginDynamicEventWithId(FNameParam Id, const TCHAR* Name, const ANSICHAR* File = nullptr, uint32 Line = 0);

	/*
	 * Output end event marker for static or dynamic event for the currently open scope.
	 */
	CORE_API static void OutputEndEvent();

	/*
	 * Output resume marker for a given spec. Must always be matched with an suspend event.
	 * @param SpecId - The unique Resume Event definition id
	 * @param TimerScopeDepth - updates the depth of the current OutputBeginEvent depth
	 */
	CORE_API static void OutputResumeEvent(uint64 SpecId, uint32& TimerScopeDepth);

	/*
	 * Output suspend event marker for the currently open resume event.
	 */
	CORE_API static void OutputSuspendEvent();

	/*
	 * Make sure all thread data has reached the destination. Can be useful to call this before entering a wait condition that might take a while.
	 */
	CORE_API static void FlushThreadBuffer();

	/*
	 * Get or create a unique event definition id for the given parameters.
	 * @param InOutSpecId - The event definition id.
	 * @param Name - The name of event
	 * @param File - The source filename
	 * @param Line - The line number in source file
	 */
	CORE_API static void GetOrCreateSpecId(uint32& InOutSpecId, const ANSICHAR* Name, const ANSICHAR* File, uint32 Line);
	CORE_API static void GetOrCreateSpecId(uint32& InOutSpecId, const TCHAR* Name, const ANSICHAR* File, uint32 Line);
	CORE_API static void GetOrCreateSpecId(uint32& InOutSpecId, FNameParam Name, const ANSICHAR* File, uint32 Line);

	class FEventScope
	{
	public:
		FORCEINLINE FEventScope(uint32 InSpecId, bool bInCondition)
			: bEnabled(bInCondition && CpuChannel)
		{
			BeginEventCommon(InSpecId);
		}

		FORCEINLINE FEventScope(uint32 InSpecId, const UE::Trace::FChannel& InChannel, bool bInCondition)
			: bEnabled(bInCondition && (CpuChannel | InChannel))
		{
			BeginEventCommon(InSpecId);
		}

		FORCEINLINE FEventScope(uint32& InOutSpecId, const ANSICHAR* InEventName, bool bInCondition, const ANSICHAR* File, uint32 Line)
			: bEnabled(bInCondition && CpuChannel)
		{
			BeginEventCommon(InOutSpecId, InEventName, File, Line);
		}

		FORCEINLINE FEventScope(uint32& InOutSpecId, const ANSICHAR* InEventName, const UE::Trace::FChannel& InChannel, bool bInCondition, const ANSICHAR* File, uint32 Line)
			: bEnabled(bInCondition && (CpuChannel | InChannel))
		{
			BeginEventCommon(InOutSpecId, InEventName, File, Line);
		}

		FORCEINLINE FEventScope(uint32& InOutSpecId, const TCHAR* InEventName, bool bInCondition, const ANSICHAR* File, uint32 Line)
			: bEnabled(bInCondition && CpuChannel)
		{
			BeginEventCommon(InOutSpecId, InEventName, File, Line);
		}

		FORCEINLINE FEventScope(uint32& InOutSpecId, const TCHAR* InEventName, const UE::Trace::FChannel& InChannel, bool bInCondition, const ANSICHAR* File, uint32 Line)
			: bEnabled(bInCondition && (CpuChannel | InChannel))
		{
			BeginEventCommon(InOutSpecId, InEventName, File, Line);
		}

		FORCEINLINE explicit FEventScope(uint32& InOutSpecId, FNameParam InEventName, bool bInCondition, const ANSICHAR* File, uint32 Line)
			: bEnabled(bInCondition && CpuChannel)
		{
			BeginEventCommon(InOutSpecId, InEventName, File, Line);
		}

		FORCEINLINE explicit FEventScope(uint32& InOutSpecId, FNameParam InEventName, const UE::Trace::FChannel& InChannel, bool bInCondition, const ANSICHAR* File, uint32 Line)
			: bEnabled(bInCondition && (CpuChannel | InChannel))
		{
			BeginEventCommon(InOutSpecId, InEventName, File, Line);
		}

		FORCEINLINE ~FEventScope()
		{
			if (bEnabled)
			{
				OutputEndEvent();
			}
		}

	private:
		FORCEINLINE void BeginEventCommon(uint32 InSpecId)
		{
			if (bEnabled)
			{
				OutputBeginEvent(InSpecId);
			}
		}

		FORCEINLINE void BeginEventCommon(uint32& InOutSpecId, const ANSICHAR* InEventName, const ANSICHAR* File, uint32 Line)
		{
			if (bEnabled)
			{
				GetOrCreateSpecId(InOutSpecId, InEventName, File, Line);
				OutputBeginEvent(FPlatformAtomics::AtomicRead_Relaxed((volatile int32*)&InOutSpecId));
			}
		}

		FORCEINLINE void BeginEventCommon(uint32& InOutSpecId, const TCHAR* InEventName, const ANSICHAR* File, uint32 Line)
		{
			if (bEnabled)
			{
				GetOrCreateSpecId(InOutSpecId, InEventName, File, Line);
				OutputBeginEvent(FPlatformAtomics::AtomicRead_Relaxed((volatile int32*)&InOutSpecId));
			}
		}

		FORCEINLINE void BeginEventCommon(uint32& InOutSpecId, FNameParam InEventName, const ANSICHAR* File, uint32 Line)
		{
			if (bEnabled)
			{
				GetOrCreateSpecId(InOutSpecId, InEventName, File, Line);
				OutputBeginEvent(FPlatformAtomics::AtomicRead_Relaxed((volatile int32*)&InOutSpecId));
			}
		}

		bool bEnabled;
	};

	struct FDynamicEventScope
	{
		FORCEINLINE FDynamicEventScope(const ANSICHAR* InEventName, bool bInCondition, const ANSICHAR* InFile = nullptr, uint32 InLine = 0)
			: bEnabled(bInCondition && CpuChannel)
		{
			if (bEnabled)
			{
				OutputBeginDynamicEvent(InEventName, InFile, InLine);
			}
		}

		FORCEINLINE FDynamicEventScope(const ANSICHAR* InEventName, const UE::Trace::FChannel& InChannel, bool bInCondition, const ANSICHAR* InFile = nullptr, uint32 InLine = 0)
			: bEnabled(bInCondition && (CpuChannel | InChannel))
		{
			if (bEnabled)
			{
				OutputBeginDynamicEvent(InEventName, InFile, InLine);
			}
		}

		FORCEINLINE FDynamicEventScope(const TCHAR* InEventName, bool bInCondition, const ANSICHAR* InFile = nullptr, uint32 InLine = 0)
			: bEnabled(bInCondition && CpuChannel)
		{
			if (bEnabled)
			{
				OutputBeginDynamicEvent(InEventName, InFile, InLine);
			}
		}

		FORCEINLINE FDynamicEventScope(const TCHAR* InEventName, const UE::Trace::FChannel& InChannel, bool bInCondition, const ANSICHAR* InFile = nullptr, uint32 InLine = 0)
			: bEnabled(bInCondition && (CpuChannel | InChannel))
		{
			if (bEnabled)
			{
				OutputBeginDynamicEvent(InEventName, InFile, InLine);
			}
		}

		FORCEINLINE explicit FDynamicEventScope(FNameParam InEventName, bool bInCondition, const ANSICHAR* InFile = nullptr, uint32 InLine = 0)
			: bEnabled(bInCondition && CpuChannel)
		{
			if (bEnabled)
			{
				OutputBeginDynamicEvent(InEventName, InFile, InLine);
			}
		}

		FORCEINLINE explicit FDynamicEventScope(FNameParam InEventName, const UE::Trace::FChannel& InChannel, bool bInCondition, const ANSICHAR* InFile = nullptr, uint32 InLine = 0)
			: bEnabled(bInCondition && (CpuChannel | InChannel))
		{
			if (bEnabled)
			{
				OutputBeginDynamicEvent(InEventName, InFile, InLine);
			}
		}

		FORCEINLINE ~FDynamicEventScope()
		{
			if (bEnabled)
			{
				OutputEndEvent();
			}
		}

		bool bEnabled;
	};

private:
	static uint8 OnAbortKey;
};

// Advanced macro for integrating e.g. stats/named events with CPU trace.
// Declares a CPU timing event for future use, conditionally and with a particular variable name for storing the id.
#define TRACE_CPUPROFILER_EVENT_DECLARE(DeclName) \
	static uint32 DeclName;

// Advanced macro for integrating e.g. stats/named events with CPU trace.
// Traces a scoped event previously declared with TRACE_CPUPROFILER_EVENT_DECLARE, conditionally.
#define TRACE_CPUPROFILER_EVENT_SCOPE_USE(DeclName, NameStr, ScopeName, Condition) \
	FCpuProfilerTrace::FEventScope ScopeName(DeclName, NameStr, Condition, __FILE__, __LINE__);

// Advanced macro for integrating e.g. stats/named events with CPU trace
// Traces a scoped event previously declared with TRACE_CPUPROFILER_EVENT_DECLARE, conditionally
#define TRACE_CPUPROFILER_EVENT_SCOPE_USE_ON_CHANNEL(DeclName, NameStr, ScopeName, Channel, Condition) \
	FCpuProfilerTrace::FEventScope ScopeName(DeclName, NameStr, Channel, Condition, __FILE__, __LINE__);

// Advanced macro that will check if CpuChannel is enabled and, if so, declares a new CPU event and starts it.
#define TRACE_CPUPROFILER_EVENT_MANUAL_START(EventNameStr) \
	if (CpuChannel) \
	{ \
		static uint32 UE_JOIN(__CpuProfilerEventSpecId, __LINE__) = FCpuProfilerTrace::OutputEventType(EventNameStr, __FILE__, __LINE__); \
		FCpuProfilerTrace::OutputBeginEvent(UE_JOIN(__CpuProfilerEventSpecId, __LINE__)); \
	}

// Advanced macro that will check if CpuChannel is enabled and, if so, ends the previously started CPU event.
#define TRACE_CPUPROFILER_EVENT_MANUAL_END() \
	if (CpuChannel) \
	{ \
		FCpuProfilerTrace::OutputEndEvent(); \
	}

// Advanced macro that can be used with TRACE_CPUPROFILER_EVENT_MANUAL_START to wrap code that should only be executed if
// the event was actually started.
#define TRACE_CPUPROFILER_EVENT_MANUAL_IS_ENABLED() \
	bool(CpuChannel)

////////////////////////////////////////////////////////////////////////////////////////////////////
// TRACE_CPUPROFILER_EVENT_SCOPE_STR* - scope name as static/constant string
////////////////////////////////////////////////////////////////////////////////////////////////////

// Conditionally trace a scoped CPU timing event providing a static/constant string (const ANSICHAR*
// or const TCHAR* or FName) as the scope name.
// It will use the CPU trace channel.
// Note: The name will be cached only once. If used with a string pointer or an FName,
//       the name should be guaranteed to not change between calls.
// Example: TRACE_CPUPROFILER_EVENT_SCOPE_STR_CONDITIONAL("My Scoped Timer A", Condition)
#define TRACE_CPUPROFILER_EVENT_SCOPE_STR_CONDITIONAL(NameStr, Condition) \
	TRACE_CPUPROFILER_EVENT_DECLARE(UE_JOIN(__CpuProfilerEventSpecId, __LINE__)); \
	TRACE_CPUPROFILER_EVENT_SCOPE_USE(UE_JOIN(__CpuProfilerEventSpecId, __LINE__), NameStr, UE_JOIN(__CpuProfilerEventScope, __LINE__), (Condition));

// Trace a scoped CPU timing event providing a static/constant string (const ANSICHAR* or
// const TCHAR* or FName) as the scope name. It will use the CPU trace channel.
// Note: The name will be cached only once. If used with a string pointer or an FName,
//       the name should be guaranteed to not change between calls.
// Example: TRACE_CPUPROFILER_EVENT_SCOPE_STR("My Scoped Timer A")
#define TRACE_CPUPROFILER_EVENT_SCOPE_STR(NameStr) \
	TRACE_CPUPROFILER_EVENT_SCOPE_STR_CONDITIONAL(NameStr, true)

// Conditionally trace a scoped CPU timing event providing a static/constant string (const ANSICHAR*
// or const TCHAR* or FName) as the scope name and a trace channel.
// Note: The name will be cached only once. If used with a string pointer or an FName,
//       the name should be guaranteed to not change between calls.
// Note: The event will be emitted only if both the given channel and the CpuChannel are enabled.
// Example: TRACE_CPUPROFILER_EVENT_SCOPE_STR_ON_CHANNEL_CONDITIONAL("My Scoped Timer A", CustomChannel, Condition)
#define TRACE_CPUPROFILER_EVENT_SCOPE_STR_ON_CHANNEL_CONDITIONAL(NameStr, Channel, Condition) \
	TRACE_CPUPROFILER_EVENT_DECLARE(UE_JOIN(__CpuProfilerEventSpecId, __LINE__)); \
	TRACE_CPUPROFILER_EVENT_SCOPE_USE_ON_CHANNEL(UE_JOIN(__CpuProfilerEventSpecId, __LINE__), NameStr, UE_JOIN(__CpuProfilerEventScope, __LINE__), Channel, (Condition));
#define TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR_CONDITIONAL(NameStr, Channel, Condition) \
	TRACE_CPUPROFILER_EVENT_SCOPE_STR_ON_CHANNEL_CONDITIONAL(NameStr, Channel, Condition)
#define TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_CONDITIONAL_STR(NameStr, Channel, Condition) \
	TRACE_CPUPROFILER_EVENT_SCOPE_STR_ON_CHANNEL_CONDITIONAL(NameStr, Channel, Condition)

// Trace a scoped CPU timing event providing a static/constant string (const ANSICHAR* or
// const TCHAR* or FName) as the scope name and a trace channel.
// Note: The name will be cached only once. If used with a string pointer or an FName,
//       the name should be guaranteed to not change between calls.
// Note: The event will be emitted only if both the given channel and the CpuChannel are enabled.
// Example: TRACE_CPUPROFILER_EVENT_SCOPE_STR_ON_CHANNEL("My Scoped Timer A", CustomChannel)
#define TRACE_CPUPROFILER_EVENT_SCOPE_STR_ON_CHANNEL(NameStr, Channel) \
	TRACE_CPUPROFILER_EVENT_SCOPE_STR_ON_CHANNEL_CONDITIONAL(NameStr, Channel, true)
#define TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR(NameStr, Channel) \
	TRACE_CPUPROFILER_EVENT_SCOPE_STR_ON_CHANNEL(NameStr, Channel)

////////////////////////////////////////////////////////////////////////////////////////////////////
// TRACE_CPUPROFILER_EVENT_SCOPE* - scope name as plain text
////////////////////////////////////////////////////////////////////////////////////////////////////

// Conditionally trace a scoped CPU timing event providing a scope name (plain text).
// It will use the CPU trace channel.
// Note: Do not use this macro with a static string because, in that case, additional quotes will
//       be added around the event scope name.
// Example: TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(MyScopedTimer::A, Condition)
#define TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(Name, Condition) \
	TRACE_CPUPROFILER_EVENT_SCOPE_STR_CONDITIONAL(#Name, (Condition))

// Trace a scoped CPU timing event providing a scope name (plain text).
// It will use the CPU trace channel.
// Note: Do not use this macro with a static string because, in that case, additional quotes will
//       be added around the event scope name.
// Example: TRACE_CPUPROFILER_EVENT_SCOPE(MyScopedTimer::A)
#define TRACE_CPUPROFILER_EVENT_SCOPE(Name) \
	TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(Name, true)

// Conditionally trace a scoped CPU timing event providing a scope name (plain text) and a trace channel.
// Note: Do not use this macro with a static string because, in that case, additional quotes will
//       be added around the event scope name.
// Note: The event will be emitted only if both the given channel and the CpuChannel are enabled.
// Example: TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_CONDITIONAL(MyScopedTimer::A, CustomChannel, Condition)
#define TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_CONDITIONAL(Name, Channel, Condition) \
	TRACE_CPUPROFILER_EVENT_SCOPE_STR_ON_CHANNEL_CONDITIONAL(#Name, Channel, (Condition))

// Trace a scoped CPU timing event providing a scope name (plain text) and a trace channel.
// Note: Do not use this macro with a static string because, in that case, additional quotes will
//       be added around the event scope name.
// Note: The event will be emitted only if both the given channel and the CpuChannel are enabled.
// Example: TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(MyScopedTimer::A, CustomChannel)
#define TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(Name, Channel) \
	TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_CONDITIONAL(Name, Channel, true)

////////////////////////////////////////////////////////////////////////////////////////////////////
// TRACE_CPUPROFILER_EVENT_SCOPE_TEXT* - scope name as dynamic string
////////////////////////////////////////////////////////////////////////////////////////////////////

// Conditionally trace a scoped CPU timing event providing a dynamic string (const ANSICHAR* or
// const TCHAR* or FName) as the scope name. It will use the CPU trace channel.
// Note: This macro has a larger overhead compared to macro that accepts a plain text name
//       or a static string. Use it only if scope name really needs to be a dynamic string.
// Example: TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(*MyScopedTimerNameString, Condition)
#define TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(Name, Condition) \
	FCpuProfilerTrace::FDynamicEventScope UE_JOIN(__CpuProfilerEventScope, __LINE__)(Name, (Condition), __FILE__, __LINE__);

// Trace a scoped CPU timing event providing a dynamic string (const ANSICHAR* or
// const TCHAR* or FName) as the scope name. It will use the CPU trace channel.
// Note: This macro has a larger overhead compared to macro that accepts a plain text name
//       or a static string. Use it only if scope name really needs to be a dynamic string.
// Example: TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*MyScopedTimerNameString)
#define TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(Name) \
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(Name, true)

// Conditionally trace a scoped CPU timing event providing a dynamic string (const ANSICHAR* or
// const TCHAR* or FName) as the scope name and a trace channel.
// Note: This macro has a larger overhead compared to macro that accepts a plain text name
//       or a static string. Use it only if scope name really needs to be a dynamic string.
// Note: The event will be emitted only if both the given channel and the CpuChannel are enabled.
// Example: TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL_CONDITIONAL(*MyScopedTimerNameString, CustomChannel, Condition)
#define TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL_CONDITIONAL(Name, Channel, Condition) \
	FCpuProfilerTrace::FDynamicEventScope UE_JOIN(__CpuProfilerEventScope, __LINE__)(Name, Channel, (Condition), __FILE__, __LINE__);

// Trace a scoped CPU timing event providing a dynamic string (const ANSICHAR* or const TCHAR* or FName)
// as the scope name and a trace channel.
// Note: This macro has a larger overhead compared to macro that accepts a plain text name
//       or a static string. Use it only if scope name really needs to be a dynamic string.
// Note: The event will be emitted only if both the given channel and the CpuChannel are enabled.
// Example: TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(*MyScopedTimerNameString, CustomChannel)
#define TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(Name, Channel) \
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL_CONDITIONAL(Name, Channel, true)

////////////////////////////////////////////////////////////////////////////////////////////////////

// Make sure all thread data has reached the destination.
// Note: Can be useful to call this before entering a wait condition that might take a while.
#define TRACE_CPUPROFILER_EVENT_FLUSH() \
	FCpuProfilerTrace::FlushThreadBuffer();

#else // CPUPROFILERTRACE_ENABLED

#define TRACE_CPUPROFILER_EVENT_DECLARE(DeclName)
#define TRACE_CPUPROFILER_EVENT_SCOPE_USE(DeclName, NameStr, ScopeName, Condition)
#define TRACE_CPUPROFILER_EVENT_SCOPE_USE_ON_CHANNEL(DeclName, NameStr, ScopeName, Channel, Condition)
#define TRACE_CPUPROFILER_EVENT_MANUAL_START(EventNameStr)
#define TRACE_CPUPROFILER_EVENT_MANUAL_END()
#define TRACE_CPUPROFILER_EVENT_MANUAL_IS_ENABLED() false
#define TRACE_CPUPROFILER_EVENT_SCOPE_STR(NameStr)
#define TRACE_CPUPROFILER_EVENT_SCOPE_STR_CONDITIONAL(NameStr, Condition)
#define TRACE_CPUPROFILER_EVENT_SCOPE_STR_ON_CHANNEL(NameStr, Channel)
#define TRACE_CPUPROFILER_EVENT_SCOPE_STR_ON_CHANNEL_CONDITIONAL(NameStr, Channel, Condition)
#define TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR(NameStr, Channel)
#define TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR_CONDITIONAL(NameStr, Channel, Condition)
#define TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_CONDITIONAL_STR(NameStr, Channel, Condition)
#define TRACE_CPUPROFILER_EVENT_SCOPE(Name)
#define TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(Name, Condition)
#define TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(Name, Channel)
#define TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_CONDITIONAL(Name, Channel, Condition)
#define TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(Name)
#define TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(Name, Condition)
#define TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(Name, Channel)
#define TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL_CONDITIONAL(Name, Channel, Condition)
#define TRACE_CPUPROFILER_EVENT_FLUSH()

#endif // CPUPROFILERTRACE_ENABLED
