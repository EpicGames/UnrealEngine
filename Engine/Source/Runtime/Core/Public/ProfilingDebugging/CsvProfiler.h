// Copyright Epic Games, Inc. All Rights Reserved.

/**
*
* A lightweight multi-threaded CSV profiler which can be used for profiling in Test/Shipping builds
*/

#pragma once

#include "Async/Future.h"
#include "Async/TaskGraphFwd.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PreprocessorHelpers.h"
#include "HAL/ThreadSafeCounter.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "ProfilingDebugging/CsvProfilerConfig.h"
#include "ProfilingDebugging/CsvProfilerTrace.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Templates/IsValidVariadicFunctionArg.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "UObject/NameTypes.h"
#include "AutoRTFM.h"
#include <atomic>

class FCsvProfiler;
class FEvent;
class FScopedCsvStat;
class FScopedCsvStatExclusive;
struct FCsvDeclaredStat;

#if CSV_PROFILER
	// Helpers
	#define CSV_CATEGORY_INDEX(CategoryName)									(_GCsvCategory_##CategoryName.Index)
	#define CSV_CATEGORY_INDEX_GLOBAL											(0)

	// Categories
	#define CSV_DEFINE_CATEGORY(CategoryName,bDefaultValue)						FCsvCategory _GCsvCategory_##CategoryName(TEXT(#CategoryName),bDefaultValue)
	#define CSV_DEFINE_CATEGORY_MODULE(Module_API,CategoryName,bDefaultValue)	FCsvCategory Module_API _GCsvCategory_##CategoryName(TEXT(#CategoryName),bDefaultValue)
	#define CSV_DECLARE_CATEGORY_EXTERN(CategoryName)							extern FCsvCategory _GCsvCategory_##CategoryName
	#define CSV_DECLARE_CATEGORY_MODULE_EXTERN(Module_API,CategoryName)			extern Module_API FCsvCategory _GCsvCategory_##CategoryName

	// Events
	#define CSV_EVENT(Category, Format, ...) \
		FCsvProfiler::RecordEventf( CSV_CATEGORY_INDEX(Category), Format, ##__VA_ARGS__ ); \
		TRACE_BOOKMARK(TEXT(UE_STRINGIZE(Category)) TEXT("/") Format, ##__VA_ARGS__)

	#define CSV_EVENT_GLOBAL(Format, ...) \
		FCsvProfiler::RecordEventf( CSV_CATEGORY_INDEX_GLOBAL, Format, ##__VA_ARGS__ ); \
		TRACE_BOOKMARK(Format, ##__VA_ARGS__)

	#define CSV_EVENT_NOLOG(Category, Format, ...) \
		FCsvProfiler::RecordEventfNoLog( CSV_CATEGORY_INDEX(Category), Format, ##__VA_ARGS__ ); \
		TRACE_BOOKMARK(TEXT(UE_STRINGIZE(Category)) TEXT("/") Format, ##__VA_ARGS__)

	#define CSV_EVENT_GLOBAL_NOLOG(Format, ...) \
		FCsvProfiler::RecordEventfNoLog( CSV_CATEGORY_INDEX_GLOBAL, Format, ##__VA_ARGS__ ); \
		TRACE_BOOKMARK(Format, ##__VA_ARGS__)

	#define CSV_REGION_BEGIN(Category, RegionName) \
		FCsvProfiler::BeginRegion(CSV_CATEGORY_INDEX(Category), RegionName);

	#define CSV_REGION_END(Category, RegionName) \
		FCsvProfiler::EndRegion(CSV_CATEGORY_INDEX(Category), RegionName);

	#define CSV_REGION_BEGIN_GLOBAL(RegionName) \
		FCsvProfiler::BeginRegion(CSV_CATEGORY_INDEX_GLOBAL, RegionName);

	#define CSV_REGION_END_GLOBAL(RegionName) \
		FCsvProfiler::EndRegion(CSV_CATEGORY_INDEX_GLOBAL, RegionName);

	// Metadata
	#define CSV_METADATA(Key,Value)									FCsvProfiler::SetMetadata( Key, Value )
	#define CSV_NON_PERSISTENT_METADATA(Key,Value)					FCsvProfiler::SetNonPersistentMetadata( Key, Value )

	// Minimal stat macros (emitted even when CSV_PROFILER_MINIMAL is 1)
	// IMPORTANT: Only a handful of key stats are expected to use these. Do not add more without very good reason!
	#define CSV_CUSTOM_STAT_MINIMAL(Category,StatName,Value,Op) 	FCsvProfiler::RecordCustomStatMinimal(#StatName, CSV_CATEGORY_INDEX(Category), Value, Op);
	#define CSV_CUSTOM_STAT_MINIMAL_GLOBAL(StatName,Value,Op) 		FCsvProfiler::RecordCustomStatMinimal(#StatName, CSV_CATEGORY_INDEX_GLOBAL, Value, Op); 

#else
	#define CSV_CATEGORY_INDEX(CategoryName)	(0)
	#define CSV_CATEGORY_INDEX_GLOBAL			(0)				
	#define CSV_DEFINE_CATEGORY(CategoryName,bDefaultValue)			
	#define CSV_DEFINE_CATEGORY_MODULE(Module_API,CategoryName,bDefaultValue)	
	#define CSV_DECLARE_CATEGORY_EXTERN(CategoryName)				
	#define CSV_DECLARE_CATEGORY_MODULE_EXTERN(Module_API,CategoryName)			
	#define CSV_EVENT(Category, Format, ...) 						
	#define CSV_EVENT_GLOBAL(Format, ...)
	#define CSV_EVENT_NOLOG(Category, Format, ...) 						
	#define CSV_EVENT_GLOBAL_NOLOG(Format, ...)
	#define CSV_REGION_BEGIN(Category, RegionName)
	#define CSV_REGION_END(Category, RegionName)
	#define CSV_REGION_BEGIN_GLOBAL(RegionName)
	#define CSV_REGION_END_GLOBAL(RegionName)
	#define CSV_METADATA(Key,Value)
	#define CSV_NON_PERSISTENT_METADATA(Key,Value)
	#define CSV_CUSTOM_STAT_MINIMAL(Category,StatName,Value,Op)
	#define CSV_CUSTOM_STAT_MINIMAL_GLOBAL(StatName,Value,Op)
#endif

// Stat macros. These are disabled if CSV_PROFILER_MINIMAL is 1 
#if CSV_PROFILER && !CSV_PROFILER_MINIMAL
	// Helpers
	#define CSV_STAT_FNAME(StatName)								(_GCsvStat_##StatName.Name)

	// Timing stats - no up-front definition
	#define CSV_SCOPED_TIMING_STAT(Category,StatName) \
		TRACE_CSV_PROFILER_INLINE_STAT(#StatName, CSV_CATEGORY_INDEX(Category)); \
		FScopedCsvStat _ScopedCsvStat_ ## StatName (#StatName, CSV_CATEGORY_INDEX(Category), "CSV_"#StatName);
	#define CSV_SCOPED_TIMING_STAT_GLOBAL(StatName) \
		TRACE_CSV_PROFILER_INLINE_STAT(#StatName, CSV_CATEGORY_INDEX_GLOBAL); \
		FScopedCsvStat _ScopedCsvStat_ ## StatName (#StatName, CSV_CATEGORY_INDEX_GLOBAL, "CSV_"#StatName);
	#define CSV_SCOPED_TIMING_STAT_CONDITIONAL(Category,StatName, Condition) \
		TRACE_CSV_PROFILER_INLINE_STAT(#StatName, CSV_CATEGORY_INDEX(Category)); \
		FScopedCsvStatConditional _ScopedCsvStat_ ## StatName (#StatName, CSV_CATEGORY_INDEX(Category), Condition, "CSV_"#StatName);
	#define CSV_SCOPED_TIMING_STAT_EXCLUSIVE(StatName) \
		TRACE_CSV_PROFILER_INLINE_STAT_EXCLUSIVE(#StatName); \
		FScopedCsvStatExclusive _ScopedCsvStatExclusive_ ## StatName (#StatName, "CSV_"#StatName);
	#define CSV_SCOPED_TIMING_STAT_EXCLUSIVE_CONDITIONAL(StatName,Condition) \
		TRACE_CSV_PROFILER_INLINE_STAT_EXCLUSIVE(#StatName); \
		FScopedCsvStatExclusiveConditional _ScopedCsvStatExclusive_ ## StatName (#StatName,Condition, "CSV_"#StatName);
	#define CSV_SCOPED_TIMING_STAT_RECURSIVE(Category,StatName) \
		TRACE_CSV_PROFILER_INLINE_STAT(#StatName, CSV_CATEGORY_INDEX(Category)); \
		static thread_local int32 _ScopedCsvStatRecursive_EntryCount_ ## StatName = 0; \
		FScopedCsvStatRecursive _ScopedCsvStatRecursive_ ## StatName (_ScopedCsvStatRecursive_EntryCount_ ## StatName, #StatName, CSV_CATEGORY_INDEX(Category), "CSV_"#StatName);
	#define CSV_SCOPED_TIMING_STAT_RECURSIVE_CONDITIONAL(Category,StatName,Condition) \
		TRACE_CSV_PROFILER_INLINE_STAT(#StatName, CSV_CATEGORY_INDEX(Category)); \
		static thread_local int32 _ScopedCsvStatRecursive_EntryCount_ ## StatName = 0; \
		FScopedCsvStatRecursiveConditional _ScopedCsvStatRecursive_ ## StatName (_ScopedCsvStatRecursive_EntryCount_ ## StatName, #StatName, CSV_CATEGORY_INDEX(Category), Condition, "CSV_"#StatName);

	// Waits
	#define CSV_SCOPED_WAIT(WaitTime)							FScopedCsvWaitConditional _ScopedCsvWait(WaitTime>0 && FCsvProfiler::IsWaitTrackingEnabledOnCurrentThread());
	#define CSV_SCOPED_WAIT_CONDITIONAL(Condition)				FScopedCsvWaitConditional _ScopedCsvWait(Condition);

	#define CSV_SCOPED_SET_WAIT_STAT(StatName) \
		TRACE_CSV_PROFILER_INLINE_STAT_EXCLUSIVE("EventWait/"#StatName); \
		FScopedCsvSetWaitStat _ScopedCsvSetWaitStat ## StatName("EventWait/"#StatName, "CsvEventWait/"#StatName, "CsvEventWait/"#StatName" (Non-CP)");

	#define CSV_SCOPED_SET_WAIT_STAT_IGNORE()						FScopedCsvSetWaitStat _ScopedCsvSetWaitStat ## StatName();

	// Custom Stats
	#define CSV_CUSTOM_STAT(Category,StatName,Value,Op) \
		UE_AUTORTFM_OPEN \
		{ \
			TRACE_CSV_PROFILER_INLINE_STAT(#StatName, CSV_CATEGORY_INDEX(Category)); \
			FCsvProfiler::RecordCustomStat(#StatName, CSV_CATEGORY_INDEX(Category), Value, Op); \
		};
	#define CSV_CUSTOM_STAT_GLOBAL(StatName,Value,Op) \
		UE_AUTORTFM_OPEN \
		{ \
			TRACE_CSV_PROFILER_INLINE_STAT(#StatName, CSV_CATEGORY_INDEX_GLOBAL); \
			FCsvProfiler::RecordCustomStat(#StatName, CSV_CATEGORY_INDEX_GLOBAL, Value, Op); \
		};

	// Stats declarations/definitions (for FName stats)
	#define CSV_DEFINE_STAT(Category,StatName)						FCsvDeclaredStat _GCsvStat_##StatName((TCHAR*)TEXT(#StatName), CSV_CATEGORY_INDEX(Category));
	#define CSV_DEFINE_STAT_GLOBAL(StatName)						FCsvDeclaredStat _GCsvStat_##StatName((TCHAR*)TEXT(#StatName), CSV_CATEGORY_INDEX_GLOBAL);
	#define CSV_DECLARE_STAT_EXTERN(Category,StatName)				extern FCsvDeclaredStat _GCsvStat_##StatName
	#define CSV_CUSTOM_STAT_DEFINED(StatName,Value,Op)				FCsvProfiler::RecordCustomStat(_GCsvStat_##StatName.Name, _GCsvStat_##StatName.CategoryIndex, Value, Op);
#else
	#define CSV_STAT_FNAME(StatName)			NAME_None							
	#define CSV_SCOPED_TIMING_STAT(Category,StatName)				
	#define CSV_SCOPED_TIMING_STAT_GLOBAL(StatName)					
	#define CSV_SCOPED_TIMING_STAT_EXCLUSIVE(StatName)
	#define CSV_SCOPED_TIMING_STAT_EXCLUSIVE_CONDITIONAL(StatName,Condition)
	#define CSV_SCOPED_TIMING_STAT_RECURSIVE(Category,StatName)
	#define CSV_SCOPED_TIMING_STAT_RECURSIVE_CONDITIONAL(Category,StatName,Condition)
	#define CSV_SCOPED_WAIT(WaitTime)
	#define CSV_SCOPED_WAIT_CONDITIONAL(Condition)
	#define CSV_SCOPED_SET_WAIT_STAT(StatName)
	#define CSV_SCOPED_SET_WAIT_STAT_IGNORE()
	#define CSV_CUSTOM_STAT(Category,StatName,Value,Op)				
	#define CSV_CUSTOM_STAT_GLOBAL(StatName,Value,Op) 				
	#define CSV_DEFINE_STAT(Category,StatName)						
	#define CSV_DEFINE_STAT_GLOBAL(StatName)						
	#define CSV_DECLARE_STAT_EXTERN(Category,StatName)				
	#define CSV_CUSTOM_STAT_DEFINED(StatName,Value,Op)				
#endif

// Shared macros. These emit Trace and CSV instrumentation. Can be used in place of TRACE_BEGIN_REGION and TRACE_END_REGION
// NOTE: CSV_TRACE_REGION_END requires a category name, unlike TRACE_END_REGION. This is in order to avoid potential naming clashes
#if CSV_PROFILER
	#define CSV_TRACE_REGION_BEGIN(RegionName, ...) \
		TRACE_BEGIN_REGION(RegionName __VA_OPT__(,) __VA_ARGS__) \
		FCsvProfiler::BeginRegion(CSV_CATEGORY_INDEX_GLOBAL, RegionName __VA_OPT__(,) __VA_ARGS__);

	#define CSV_TRACE_REGION_END(RegionName, ...) \
		FCsvProfiler::EndRegion(CSV_CATEGORY_INDEX_GLOBAL, RegionName __VA_OPT__(,) __VA_ARGS__); \
		TRACE_END_REGION(RegionName)
#else // CSV_PROFILER
	#define CSV_TRACE_REGION_BEGIN(RegionName, ...) \
		TRACE_BEGIN_REGION(RegionName __VA_OPT__(,) __VA_ARGS__)

	#define CSV_TRACE_REGION_END(RegionName, ...) \
		TRACE_END_REGION(RegionName)
#endif // !CSV_PROFILER

#if CSV_PROFILER

class FCsvProfilerFrame;
class FCsvProfilerProcessingThread;
class FCsvProfilerThreadData;
class FName;
struct FCsvPersistentCustomStats;

enum class ECsvCustomStatOp : uint8
{
	Set,
	Min,
	Max,
	Accumulate,
};

enum class ECsvCommandType : uint8
{
	Start,
	Stop,
	Count
};

struct FCsvCategory;

struct FCsvDeclaredStat
{
	FCsvDeclaredStat(const TCHAR* InNameString, uint32 InCategoryIndex) 
		: Name(InNameString)
		, CategoryIndex(InCategoryIndex) 
	{
		TRACE_CSV_PROFILER_DECLARED_STAT(Name, InCategoryIndex);
	}

	FName Name;
	uint32 CategoryIndex;
};

enum class ECsvProfilerFlags
{
	None = 0x0,
	WriteCompletionFile = 0x1,
	CompressOutput = 0x2,
	BootCapture = 0x4
};
ENUM_CLASS_FLAGS(ECsvProfilerFlags);

struct FCsvCaptureCommand
{
	FCsvCaptureCommand()
		: CommandType(ECsvCommandType::Count)
		, FrameRequested(-1)
		, Value(-1)
	{}

	FCsvCaptureCommand(ECsvCommandType InCommandType, uint32 InFrameRequested, uint32 InValue, const FString& InDestinationFolder, const FString& InFilename, ECsvProfilerFlags InFlags)
		: CommandType(InCommandType)
		, FrameRequested(InFrameRequested)
		, Value(InValue)
		, DestinationFolder(InDestinationFolder)
		, Filename(InFilename)
		, Flags(InFlags)
	{}

	FCsvCaptureCommand(ECsvCommandType InCommandType, uint32 InFrameRequested, TPromise<FString>* InCompletion, TSharedFuture<FString> InFuture)
		: CommandType(InCommandType)
		, FrameRequested(InFrameRequested)
		, Completion(MoveTemp(InCompletion))
		, Future(InFuture)
	{}

	ECsvCommandType CommandType;
	uint32 FrameRequested;
	uint32 Value;
	FString DestinationFolder;
	FString Filename;
	ECsvProfilerFlags Flags;
	TPromise<FString>* Completion;
	TSharedFuture<FString> Future;
};

//
// Persistent custom stats
// 
enum class ECsvPersistentCustomStatType : uint8
{
	Float,
	Int,
	Count
};

class FCsvPersistentCustomStatBase
{
public:
	FCsvPersistentCustomStatBase(FName InName, uint32 InCategoryIndex, bool bInResetEachFrame, ECsvPersistentCustomStatType InStatType)
		: Name(InName), CategoryIndex(InCategoryIndex), bResetEachFrame(bInResetEachFrame), StatType(InStatType)
	{
	}

	ECsvPersistentCustomStatType GetStatType() const
	{
		return StatType;
	}
protected:
	FName Name;
	uint32 CategoryIndex;
	bool bResetEachFrame;
	ECsvPersistentCustomStatType StatType;
};


template <class T> 
class TCsvPersistentCustomStat : public FCsvPersistentCustomStatBase
{
	friend struct FCsvPersistentCustomStats;
private:
	std::atomic<T> Value;
};


template <>
class TCsvPersistentCustomStat<float> : public FCsvPersistentCustomStatBase
{
	friend struct FCsvPersistentCustomStats;
public:
	TCsvPersistentCustomStat<float>(FName InName, uint32 InCategoryIndex, bool bInResetEachFrame)
		: FCsvPersistentCustomStatBase(InName, InCategoryIndex, bInResetEachFrame, ECsvPersistentCustomStatType::Float)
		, Value(0.0f)
	{}

	float Add(float Rhs)
	{
		float Previous = Value.load(std::memory_order_consume);
		float Desired = Previous + Rhs;
		while (!Value.compare_exchange_weak(Previous, Desired, std::memory_order_release, std::memory_order_consume))
		{
			Desired = Previous + Rhs;
		}
		return Desired;
	}
	float Sub(float Rhs)
	{
		return Add(-Rhs);
	}
	void Set(float NewVal)
	{
		return Value.store(NewVal, std::memory_order_relaxed);
	}
	float GetValue()
	{
		return Value.load();
	}
	static ECsvPersistentCustomStatType GetClassStatType() { return ECsvPersistentCustomStatType::Float; }
protected:
	std::atomic<float> Value;
};


template <>
class TCsvPersistentCustomStat<int32> : public FCsvPersistentCustomStatBase
{
	friend struct FCsvPersistentCustomStats;
public:
	TCsvPersistentCustomStat<int32>(FName InName, uint32 InCategoryIndex, bool bInResetEachFrame)
		: FCsvPersistentCustomStatBase(InName, InCategoryIndex, bInResetEachFrame, ECsvPersistentCustomStatType::Int)
		, Value(0)
	{}

	int32 Add(int32 Rhs)
	{
		return Value.fetch_add(Rhs, std::memory_order_relaxed);
	}
	int32 Sub(int32 Rhs)
	{
		return Value.fetch_sub(Rhs, std::memory_order_relaxed);
	}
	void Set(int32 NewVal)
	{
		Value.store(NewVal, std::memory_order_relaxed);
	}
	int32 GetValue()
	{
		return Value.load();
	}
	static ECsvPersistentCustomStatType GetClassStatType() { return ECsvPersistentCustomStatType::Int; }
protected:
	std::atomic<int> Value;
};

/**
* Processing stats
*/
struct FCsvProcessThreadDataStats
{
	uint32 TimestampCount = 0;
	uint32 CustomStatCount = 0;
	uint32 EventCount = 0;
	uint64 MemoryUseBytes = 0;
};

/**
* FCsvProfiler class. This manages recording and reporting all for CSV stats
*/
class FCsvProfiler
{
	friend class FCsvProfilerProcessingThread;
	friend class FCsvProfilerThreadData;
	friend struct FCsvCategory;
public:
	FCsvProfiler();
	~FCsvProfiler();
	static CORE_API FCsvProfiler* Get();

	CORE_API void Init();

	CORE_API static void PreInit(const TCHAR* Commandline);

	/** Begin static interface (used by macros)*/
	/** Push/pop events */
	CORE_API static void BeginStat(const char* StatName, uint32 CategoryIndex, const char* NamedEventName = nullptr);
	CORE_API static void BeginStat(const FName& StatName, uint32 CategoryIndex);
	CORE_API static void EndStat(const char* StatName, uint32 CategoryIndex);
	CORE_API static void EndStat(const FName& StatName, uint32 CategoryIndex);

	CORE_API static void BeginExclusiveStat(const char * StatName, const char* NamedEventName = nullptr);
	CORE_API static void EndExclusiveStat(const char * StatName);

	CORE_API static void RecordCustomStat(const char * StatName, uint32 CategoryIndex, float Value, const ECsvCustomStatOp CustomStatOp);
	CORE_API static void RecordCustomStat(const FName& StatName, uint32 CategoryIndex, float Value, const ECsvCustomStatOp CustomStatOp);
	CORE_API static void RecordCustomStat(const char* StatName, uint32 CategoryIndex, double Value, const ECsvCustomStatOp CustomStatOp);
	CORE_API static void RecordCustomStat(const FName& StatName, uint32 CategoryIndex, double Value, const ECsvCustomStatOp CustomStatOp);
	CORE_API static void RecordCustomStat(const char * StatName, uint32 CategoryIndex, int32 Value, const ECsvCustomStatOp CustomStatOp);
	CORE_API static void RecordCustomStat(const FName& StatName, uint32 CategoryIndex, int32 Value, const ECsvCustomStatOp CustomStatOp);

	CORE_API static void RecordEndOfPipeCustomStat(const FName& StatName, uint32 CategoryIndex, double Value, const ECsvCustomStatOp CustomStatOp);

	// Warning: Don't use this unless the stat is one of a handful of key stats that we want to capture in CSV_PROFILER_MINIMAL mode
	CORE_API static void RecordCustomStatMinimal(const char* StatName, uint32 CategoryIndex, float Value, const ECsvCustomStatOp CustomStatOp);
	CORE_API static void RecordCustomStatMinimal(const char* StatName, uint32 CategoryIndex, int32 Value, const ECsvCustomStatOp CustomStatOp);
	CORE_API static void RecordCustomStatMinimal(const char* StatName, uint32 CategoryIndex, double Value, const ECsvCustomStatOp CustomStatOp);

	CORE_API static void RecordEvent(int32 CategoryIndex, const FString& EventText, bool bLog=true, bool bAllowDelegate=true);
	CORE_API static void RecordEventAtFrameStart(int32 CategoryIndex, const FString& EventText, bool bLog = true);
	CORE_API static void RecordEventAtTimestamp(int32 CategoryIndex, const FString& EventText, uint64 Cycles64, bool bLog = true);

	/** Metadata values set with this function will persist between captures. */
	CORE_API static void SetMetadata(const TCHAR* Key, const TCHAR* Value);
	/** Metadata values set with this function will be cleared after the capture ends. */
	CORE_API static void SetNonPersistentMetadata(const TCHAR* Key, const TCHAR* Value);
	
	static CORE_API int32 RegisterCategory(const FString& Name, bool bEnableByDefault, bool bIsGlobal);
	static CORE_API int32 GetCategoryIndex(const FString& Name);

	template <typename FmtType, typename... Types>
	inline static void RecordEventf(int32 CategoryIndex, const FmtType& Fmt, Types... Args)
	{
		UE_AUTORTFM_OPEN
		{
			static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
			static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FCsvProfiler::RecordEventf");
			RecordEventfInternal(CategoryIndex, true, (const TCHAR*)Fmt, Args...);
		};
	}

	template <typename FmtType, typename... Types>
	inline static void RecordEventfNoLog(int32 CategoryIndex, const FmtType& Fmt, Types... Args)
	{
		UE_AUTORTFM_OPEN
		{
			static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
			static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FCsvProfiler::RecordEventfNoLog");
			RecordEventfInternal(CategoryIndex, false, (const TCHAR*)Fmt, Args...);
		};
	}

	CORE_API static void BeginSetWaitStat(const char* StatName, const char* FormattedStatName, const char* FormattedStatNameNonCP);
	CORE_API static void EndSetWaitStat();

	CORE_API static void BeginWait();
	CORE_API static void EndWait();

	CORE_API static void BeginRegion(int32 CategoryIndex, const TCHAR* RegionName, const TCHAR* SubCategoryName = nullptr);
	CORE_API static void EndRegion(int32 CategoryIndex, const TCHAR* RegionName, const TCHAR* SubCategoryName = nullptr);

	CORE_API static FString GetDefaultDirectory();

	CORE_API static bool IsCapturing();
	CORE_API static bool IsCapturing_Renderthread();

	/** Singleton interface */
	CORE_API bool IsWritingFile() const;
	CORE_API bool IsEndCapturePending() const;

	CORE_API int32 GetCaptureFrameNumber() const;
	CORE_API int32 GetCaptureFrameNumberRT() const;
	CORE_API int32 GetNumFrameToCaptureOnEvent() const;
	CORE_API const FGuid& GetCsvID() const;
	CORE_API uint64	GetNumBytesWritten() const;

	CORE_API bool EnableCategoryByString(const FString& CategoryName) const;
	CORE_API void EnableCategoryByIndex(uint32 CategoryIndex, bool bEnable) const;
	CORE_API bool IsCategoryEnabled(uint32 CategoryIndex) const;

	/** Per-frame update */
	CORE_API void BeginFrame();
	CORE_API void EndFrame();

	/** Begin Capture */
	CORE_API void BeginCapture(int InNumFramesToCapture = -1,
		const FString& InDestinationFolder = FString(),
		const FString& InFilename = FString(),
		ECsvProfilerFlags InFlags = ECsvProfilerFlags::None);

	/**
	 * End Capture
	 * EventToSignal is optional. If provided, the CSV profiler will signal the event when the async file write is complete.
	 * The returned TFuture can be waited on to retrieve the filename of the csv file that was written to disk.
	 */
	CORE_API TSharedFuture<FString> EndCapture(FGraphEventRef EventToSignal = {});

	/** Called at the end of the first frame after forking */
	CORE_API void OnEndFramePostFork();

	/** Renderthread begin/end frame */
	CORE_API void BeginFrameRT();
	CORE_API void EndFrameRT();

	CORE_API void BeginFrameEOP();

	/** Called from the engine tick at the point where we should compute the current frame time. */
	CORE_API void UpdateFrameTime();

	CORE_API void SetDeviceProfileName(FString InDeviceProfileName);

	FString GetOutputFilename() const { return OutputFilename; }
	CORE_API TMap<FString, FString> GetMetadataMapCopy();

	CORE_API static bool IsWaitTrackingEnabledOnCurrentThread();

	CORE_API void GetFrameExecCommands(TArray<FString>& OutFrameCommands);

	/** Called right before we start capturing. */
	DECLARE_MULTICAST_DELEGATE(FOnCSVProfileStart);
	FOnCSVProfileStart& OnCSVProfileStart() { return OnCSVProfileStartDelegate; }

	/** Called when csv frame 0 begins its capture. This is when CsvEvents and CustomStats will start being collected. */
	DECLARE_MULTICAST_DELEGATE(FOnCSVProfileFirstFrame);
	FOnCSVProfileFirstFrame& OnCSVProfileFirstFrame() { return OnCSVProfileFirstFrameDelegate; }

	/** Called when the capture is requested to end, allowing any final information to be written (eg. metadata). */
	DECLARE_MULTICAST_DELEGATE(FOnCSVProfileEndRequested);
	FOnCSVProfileEndRequested& OnCSVProfileEndRequested() { return OnCSVProfileEndRequestedDelegate; }

	DECLARE_MULTICAST_DELEGATE(FOnCSVProfileEnd);
	FOnCSVProfileEnd& OnCSVProfileEnd() { return OnCSVProfileEndDelegate; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnCSVProfileFinished, const FString& /*Filename */);
	FOnCSVProfileFinished& OnCSVProfileFinished() { return OnCSVProfileFinishedDelegate; }

	/** Called at the end of the frame when capturing, before end frame processing completes */
	DECLARE_MULTICAST_DELEGATE(FOnCSVProfileEndFrame);
	FOnCSVProfileEndFrame& OnCSVProfileEndFrame() { return OnCSVProfileEndFrameDelegate; }

	/** Called when an event is emitted. 
		NOTE: this will only fire if CSV_PROFILER_ALLOW_DEBUG_FEATURES is enabled */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnCSVEvent, const FString& /* CategoryName */, const FString& /* EventText */);
	FOnCSVEvent& OnCSVProfileEvent() { return OnCSVProfileEventDelegate; }

	void SetRenderThreadId(uint32 InRenderThreadId)
	{
		RenderThreadId = InRenderThreadId;
	}

	void SetRHIThreadId(uint32 InRHIThreadId)
	{
		RHIThreadId = InRHIThreadId;
	}

	// Persistent custom stat methods. These are pre-registered custom stats whose value can persist over multiple frames
	// They are lower overhead than normal custom stats because accumulate ops don't require memory allocation
	CORE_API TCsvPersistentCustomStat<int32>* GetOrCreatePersistentCustomStatInt(FName Name, int32 CategoryIndex = CSV_CATEGORY_INDEX_GLOBAL, bool bResetEachFrame = false);
	CORE_API TCsvPersistentCustomStat<float>* GetOrCreatePersistentCustomStatFloat(FName Name, int32 CategoryIndex = CSV_CATEGORY_INDEX_GLOBAL, bool bResetEachFrame = false);

private:
	CORE_API static void VARARGS RecordEventfInternal(int32 CategoryIndex, bool bLog, const TCHAR* Fmt, ...);

	void InitInternal();
	
	static void RecordRegionEventInternal(const TCHAR* CategoryName, const TCHAR* RegionName, const TCHAR* SubCategoryName, bool bBegin);

	enum class EMetadataPersistenceType : int8
	{
		Persistent,
		NonPersistent
	};

	void SetMetadataInternal(const TCHAR* Key, const TCHAR* Value, bool bSanitize=true, EMetadataPersistenceType PersistenceType = EMetadataPersistenceType::Persistent);

	void FinalizeCsvFile();

	float ProcessStatData(FCsvProcessThreadDataStats* OutNumStatEntriesProcessed = nullptr);

	void BeginCaptureInternal(const FCsvCaptureCommand& CurrentCommand);

	bool TryEndCaptureInternal(const FCsvCaptureCommand& CurrentCommand);

	int32 NumFramesToCapture;
	int32 CaptureFrameNumber;
	int32 CaptureFrameNumberRT;
	int32 CaptureOnEventFrameCount;

	FGuid CsvGUID;

	bool bInsertEndFrameAtFrameStart;
	bool bNamedEventsWasEnabled;

	uint64 LastEndFrameTimestamp;
	uint32 CaptureEndFrameCount;

	double CaptureStartTime;

	FString OutputFilename;
	TQueue<FCsvCaptureCommand> CommandQueue;
	FCsvProfilerProcessingThread* ProcessingThread;

	FEvent* FileWriteBlockingEvent;
	FThreadSafeCounter IsShuttingDown;

	TMap<FString, FString> MetadataMap;
	TMap<FString, FString> NonPersistentMetadataMap;
	TQueue<TMap<FString, FString>> MetadataQueue;
	FTransactionallySafeCriticalSection MetadataCS;

	class FCsvStreamWriter* CsvWriter;

	ECsvProfilerFlags CurrentFlags;

	FOnCSVProfileStart OnCSVProfileStartDelegate;
	FOnCSVProfileFirstFrame OnCSVProfileFirstFrameDelegate;
	FOnCSVProfileEndRequested OnCSVProfileEndRequestedDelegate;
	FOnCSVProfileEnd OnCSVProfileEndDelegate;
	FOnCSVProfileEndFrame OnCSVProfileEndFrameDelegate;
	FOnCSVProfileFinished OnCSVProfileFinishedDelegate;
	FOnCSVEvent OnCSVProfileEventDelegate;

	std::atomic<uint32> RenderThreadId{ 0 };
	std::atomic<uint32> RHIThreadId{ 0 };
};

class FScopedCsvStat
{
public:
	FScopedCsvStat(const char * InStatName, uint32 InCategoryIndex, const char * InNamedEventName = nullptr)
		: StatName(InStatName)
		, CategoryIndex(InCategoryIndex)
	{
		UE_AUTORTFM_OPEN{ FCsvProfiler::BeginStat(StatName, CategoryIndex, InNamedEventName); };
		AutoRTFM::PushOnAbortHandler(this, [SN = StatName, CI = CategoryIndex](){ FCsvProfiler::EndStat(SN, CI); });
	}

	~FScopedCsvStat()
	{
		AutoRTFM::PopOnAbortHandler(this);
		UE_AUTORTFM_OPEN{ FCsvProfiler::EndStat(StatName, CategoryIndex); };
	}
	const char * StatName;
	uint32 CategoryIndex;
};

class FScopedCsvStatConditional
{
public:
	FScopedCsvStatConditional(const char* InStatName, uint32 InCategoryIndex, bool bInCondition, const char* InNamedEventName = nullptr)
		: StatName(InStatName)
		, CategoryIndex(InCategoryIndex)
		, bCondition(bInCondition)
	{
		if (bCondition)
		{
			FCsvProfiler::BeginStat(StatName, CategoryIndex, InNamedEventName);
		}
	}

	~FScopedCsvStatConditional()
	{
		if (bCondition)
		{
			FCsvProfiler::EndStat(StatName, CategoryIndex);
		}
	}

	const char* StatName;
	uint32 CategoryIndex;
	bool bCondition;
};

class FScopedCsvStatExclusive 
{
public:
	FScopedCsvStatExclusive(const char * InStatName, const char* InNamedEventName = nullptr)
		: StatName(InStatName)
	{
		UE_AUTORTFM_OPEN{ FCsvProfiler::BeginExclusiveStat(StatName, InNamedEventName); };
		AutoRTFM::PushOnAbortHandler(this, [SN = StatName](){ FCsvProfiler::EndExclusiveStat(SN); });
	}

	~FScopedCsvStatExclusive()
	{
		AutoRTFM::PopOnAbortHandler(this);
		UE_AUTORTFM_OPEN{ FCsvProfiler::EndExclusiveStat(StatName); };
	}
	const char * StatName;
};

class FScopedCsvStatExclusiveConditional
{
public:
	FScopedCsvStatExclusiveConditional(const char * InStatName, bool bInCondition, const char* InNamedEventName = nullptr)
		: StatName(InStatName)
		, bCondition(bInCondition)
	{
		if (bCondition)
		{
			FCsvProfiler::BeginExclusiveStat(StatName, InNamedEventName);
		}
	}

	~FScopedCsvStatExclusiveConditional()
	{
		if (bCondition)
		{
			FCsvProfiler::EndExclusiveStat(StatName);
		}
	}
	const char * StatName;
	bool bCondition;
};

class FScopedCsvStatRecursive
{
	const char* StatName;
	uint32 CategoryIndex;
	int32& EntryCounter;
public:
	FScopedCsvStatRecursive(int32& InEntryCounter, const char* InStatName, uint32 InCategoryIndex, const char* InNamedEventName = nullptr)
		: StatName(InStatName)
		, CategoryIndex(InCategoryIndex)
		, EntryCounter(InEntryCounter)
	{
		++EntryCounter; // this needs to happen before BeginStat in case BeginStat causes reentry
		if (EntryCounter == 1)
		{
			FCsvProfiler::BeginStat(StatName, CategoryIndex, InNamedEventName);
		}
	}
	~FScopedCsvStatRecursive()
	{
		if (EntryCounter == 1)
		{
			FCsvProfiler::EndStat(StatName, CategoryIndex);
		}
		--EntryCounter;
	}
};

class FScopedCsvStatRecursiveConditional
{
	const char* StatName;
	uint32 CategoryIndex;
	int32& EntryCounter;
	bool bCondition;
public:
	FScopedCsvStatRecursiveConditional(int32& InEntryCounter, const char* InStatName, uint32 InCategoryIndex, bool bInCondition, const char* InNamedEventName = nullptr)
		: StatName(InStatName)
		, CategoryIndex(InCategoryIndex)
		, EntryCounter(InEntryCounter)
		, bCondition(bInCondition)
	{
		if (bCondition)
		{
			++EntryCounter; // this needs to happen before BeginStat in case BeginStat causes reentry
			if (EntryCounter == 1)
			{
				FCsvProfiler::BeginStat(StatName, CategoryIndex, InNamedEventName);
			}
		}
	}
	~FScopedCsvStatRecursiveConditional()
	{
		if (bCondition)
		{
			if (EntryCounter == 1)
			{
				FCsvProfiler::EndStat(StatName, CategoryIndex);
			}
			--EntryCounter;
		}
	}
};

class FScopedCsvWaitConditional
{
public:
	FScopedCsvWaitConditional(bool bInCondition)
		: bCondition(bInCondition)
	{
		if (bCondition)
		{
			FCsvProfiler::BeginWait();
		}
	}

	~FScopedCsvWaitConditional()
	{
		if (bCondition)
		{
			FCsvProfiler::EndWait();
		}
	}
	bool bCondition;
};


class FScopedCsvSetWaitStat
{
public:
	FScopedCsvSetWaitStat(
		const char* InStatName = nullptr,
		const char* InFormattedStatName = nullptr,
		const char* InFormattedStatNameNonCP = nullptr)
		: StatName(InStatName)
		, FormattedStatName(InFormattedStatName)
		, FormattedStatNameNonCP(InFormattedStatNameNonCP)
	{
		FCsvProfiler::BeginSetWaitStat(StatName, FormattedStatName, FormattedStatNameNonCP);
	}

	~FScopedCsvSetWaitStat()
	{
		FCsvProfiler::EndSetWaitStat();
	}

	const char* StatName;
	const char* FormattedStatName;
	const char* FormattedStatNameNonCP;
};

struct FCsvCategory
{
	FCsvCategory() : Index(-1) {}
	FCsvCategory(const TCHAR* CategoryString, bool bDefaultValue, bool bIsGlobal = false)
	{
		Name = CategoryString;
		Index = FCsvProfiler::RegisterCategory(Name, bDefaultValue, bIsGlobal);
	}

	uint32 Index;
	FString Name;
};


CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Exclusive);


#endif //CSV_PROFILER
