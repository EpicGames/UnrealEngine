// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

// Whether to allow the CSV profiler in shipping builds.
// Enable in a .Target.cs file if required.
#ifndef CSV_PROFILER_ENABLE_IN_SHIPPING
#define CSV_PROFILER_ENABLE_IN_SHIPPING 0
#endif

// Enables command line switches and unit tests of the CSV profiler.
// The default disables these features in a shipping build, but a .Target.cs file can override this.
#ifndef CSV_PROFILER_ALLOW_DEBUG_FEATURES
#define CSV_PROFILER_ALLOW_DEBUG_FEATURES (!UE_BUILD_SHIPPING)
#endif

#ifndef CSV_PROFILER_USE_CUSTOM_FRAME_TIMINGS
#define CSV_PROFILER_USE_CUSTOM_FRAME_TIMINGS 0
#endif

// Enable this to run the CSV profiler with near-zero instrumentation overhead. Only a handful of key stats will be captured
#ifndef CSV_PROFILER_MINIMAL
#define CSV_PROFILER_MINIMAL 0
#endif


// CSV_PROFILER default enabling rules, if not specified explicitly in <Program>.Target.cs GlobalDefinitions
#ifndef CSV_PROFILER
	#define CSV_PROFILER (WITH_ENGINE && (!UE_BUILD_SHIPPING || CSV_PROFILER_ENABLE_IN_SHIPPING))
#endif

// CSV_PROFILER_STATS is disabled if CSV_PROFILER_MINIMAL is enabled. Profiling subsystems should be predicated on this rather than CSV_PROFILER
#define CSV_PROFILER_STATS (CSV_PROFILER && !CSV_PROFILER_MINIMAL)