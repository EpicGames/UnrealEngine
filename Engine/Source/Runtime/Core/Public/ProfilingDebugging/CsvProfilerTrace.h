// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PreprocessorHelpers.h"
#include "Misc/Build.h"
#include "ProfilingDebugging/CsvProfilerConfig.h"
#include "Trace/Config.h"

// Note: Enabling this adds about 15% to the instrumentation overhead of the CSV profiler, even if Insights is not capturing
#ifndef CSVPROFILERTRACE_ENABLED
#if UE_TRACE_ENABLED && !UE_BUILD_SHIPPING
#define CSVPROFILERTRACE_ENABLED (WITH_ENGINE && CSV_PROFILER_STATS)
#else
#define CSVPROFILERTRACE_ENABLED 0
#endif
#endif

class FName;

#if CSVPROFILERTRACE_ENABLED

struct FCsvProfilerTrace
{
	static void OutputRegisterCategory(int32 Index, const TCHAR* Name);
	CORE_API static void OutputInlineStat(const char* StatName, int32 CategoryIndex);
	CORE_API static void OutputInlineStatExclusive(const char* StatName);
	CORE_API static void OutputDeclaredStat(const FName& StatName, int32 CategoryIndex);
	static void OutputBeginStat(const char* StatName, int32 CategoryIndex, uint64 Cycles);
	static void OutputBeginStat(const FName& StatName, int32 CategoryIndex, uint64 Cycles);
	static void OutputEndStat(const char* StatName, int32 CategoryIndex, uint64 Cycles);
	static void OutputEndStat(const FName& StatName, int32 CategoryIndex, uint64 Cycles);
	static void OutputBeginExclusiveStat(const char* StatName, int32 CategoryIndex, uint64 Cycles);
	static void OutputEndExclusiveStat(const char* StatName, int32 CategoryIndex, uint64 Cycles);
	static void OutputCustomStat(const char* StatName, int32 CategoryIndex, int32 Value, uint8 OpType, uint64 Cycles);
	static void OutputCustomStat(const FName& StatName, int32 CategoryIndex, int32 Value, uint8 OpType, uint64 Cycles);
	static void OutputCustomStat(const char* StatName, int32 CategoryIndex, float Value, uint8 OpType, uint64 Cycles);
	static void OutputCustomStat(const FName& StatName, int32 CategoryIndex, float Value, uint8 OpType, uint64 Cycles);
	static void OutputEvent(const TCHAR* Text, int32 CategoryIndex, uint64 Cycles);
	static void OutputBeginCapture(const TCHAR* Filename, uint32 RenderThreadId, uint32 RHIThreadId, const char* DefaultWaitStatName, bool bEnableCounts);
	static void OutputEndCapture();
	static void OutputMetadata(const TCHAR* Key, const TCHAR* Value);
};

#define TRACE_CSV_PROFILER_REGISTER_CATEGORY(Index, Name) \
	FCsvProfilerTrace::OutputRegisterCategory(Index, Name);

#define TRACE_CSV_PROFILER_INLINE_STAT(StatName, CategoryIndex) \
	static TSAN_ATOMIC(bool) UE_JOIN(__CsvProfilerStat, __LINE__); \
	if (!UE_JOIN(__CsvProfilerStat, __LINE__)) { \
		FCsvProfilerTrace::OutputInlineStat(StatName, CategoryIndex); \
		UE_JOIN(__CsvProfilerStat, __LINE__) = true; \
	}

#define TRACE_CSV_PROFILER_INLINE_STAT_EXCLUSIVE(StatName) \
	static TSAN_ATOMIC(bool) UE_JOIN(__CsvProfilerStat, __LINE__); \
	if (!UE_JOIN(__CsvProfilerStat, __LINE__)) { \
		FCsvProfilerTrace::OutputInlineStatExclusive(StatName); \
		UE_JOIN(__CsvProfilerStat, __LINE__) = true; \
	}

#define TRACE_CSV_PROFILER_DECLARED_STAT(StatName, CategoryIndex) \
	FCsvProfilerTrace::OutputDeclaredStat(StatName, CategoryIndex);
	
#define TRACE_CSV_PROFILER_BEGIN_STAT(StatName, CategoryIndex, Cycles) \
	FCsvProfilerTrace::OutputBeginStat(StatName, CategoryIndex, Cycles);

#define TRACE_CSV_PROFILER_END_STAT(StatName, CategoryIndex, Cycles) \
	FCsvProfilerTrace::OutputEndStat(StatName, CategoryIndex, Cycles);

#define TRACE_CSV_PROFILER_BEGIN_EXCLUSIVE_STAT(StatName, CategoryIndex, Cycles) \
	FCsvProfilerTrace::OutputBeginExclusiveStat(StatName, CategoryIndex, Cycles);

#define TRACE_CSV_PROFILER_END_EXCLUSIVE_STAT(StatName, CategoryIndex, Cycles) \
	FCsvProfilerTrace::OutputEndExclusiveStat(StatName, CategoryIndex, Cycles);

#define TRACE_CSV_PROFILER_CUSTOM_STAT(StatName, CategoryIndex, Value, OpType, Cycles) \
	FCsvProfilerTrace::OutputCustomStat(StatName, CategoryIndex, Value, OpType, Cycles);

#define TRACE_CSV_PROFILER_EVENT(Text, CategoryIndex, Cycles) \
	FCsvProfilerTrace::OutputEvent(Text, CategoryIndex, Cycles);

#define TRACE_CSV_PROFILER_BEGIN_CAPTURE(Filename, RenderThreadId, RHIThreadId, DefaultWaitStatName, EnableCounts) \
	FCsvProfilerTrace::OutputBeginCapture(Filename, RenderThreadId, RHIThreadId, DefaultWaitStatName, EnableCounts);

#define TRACE_CSV_PROFILER_END_CAPTURE() \
	FCsvProfilerTrace::OutputEndCapture();

#define TRACE_CSV_PROFILER_METADATA(Key, Value) \
	FCsvProfilerTrace::OutputMetadata(Key, Value);

#else

#define TRACE_CSV_PROFILER_REGISTER_CATEGORY(...)
#define TRACE_CSV_PROFILER_INLINE_STAT(...)
#define TRACE_CSV_PROFILER_INLINE_STAT_EXCLUSIVE(...)
#define TRACE_CSV_PROFILER_DECLARED_STAT(...)
#define TRACE_CSV_PROFILER_BEGIN_STAT(...)
#define TRACE_CSV_PROFILER_END_STAT(...)
#define TRACE_CSV_PROFILER_BEGIN_EXCLUSIVE_STAT(...)
#define TRACE_CSV_PROFILER_END_EXCLUSIVE_STAT(...)
#define TRACE_CSV_PROFILER_CUSTOM_STAT(...)
#define TRACE_CSV_PROFILER_EVENT(...)
#define TRACE_CSV_PROFILER_BEGIN_CAPTURE(...)
#define TRACE_CSV_PROFILER_END_CAPTURE(...)
#define TRACE_CSV_PROFILER_METADATA(...)

#endif
