// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DebuggingDefines.h: Defines used for toggling debugging code throughout the engine.
=============================================================================*/

#pragma once

#include "CoreTypes.h"

// compiles out code used for debugging issues with distributed cooking
#define DEBUG_DISTRIBUTED_COOKING 0









