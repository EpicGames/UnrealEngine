// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiagnosticTable.h: Diagnostic table implementation
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Misc/VarArgs.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/UnrealString.h"
#include "Misc/DateTime.h"
#include "HAL/PlatformProcess.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"

#if ALLOW_DEBUG_FILES

/**
 * Encapsulates writing a table of data to a CSV file that is output by the engine for diagnostic purposes.
 */
class FDiagnosticTableWriterCSV
{
public:

	/** Initialization constructor. */
	FDiagnosticTableWriterCSV(FArchive* InOutputStream)
	:	OutputStream(InOutputStream)
	{}

	/** Destructor; closes the output stream. */
	~FDiagnosticTableWriterCSV()
	{
		// Ensure the user flushed the last row to the output.
		check(!CurrentRow.Len());

		// Close the output stream.
		Close();
	}

	/** Adds a column to the current row. */
	void AddColumn(const TCHAR* Format,...)
	{
		check(OutputStream);

		// Format the input string.
		TCHAR ColumnText[4096];
		GET_TYPED_VARARGS(TCHAR,ColumnText,UE_ARRAY_COUNT(ColumnText),UE_ARRAY_COUNT(ColumnText)-1,Format,Format);

		// If this isn't the first column in the row, add a comma to separate the columns.
		if(CurrentRow.Len())
		{
			CurrentRow += TEXT(',');
		}

		// Replace quotes in the column with doubles quotes.
		FString EscapedText = ColumnText;
		EscapedText = EscapedText.Replace(TEXT("\""),TEXT("\"\""));

		// Append the enquoted column to the current row.
		CurrentRow += TEXT('\"');
		CurrentRow += EscapedText;
		CurrentRow += TEXT('\"');
	}

	/** Advances to the next row. */
	void CycleRow()
	{
		check(OutputStream);

		// Write the current row to the output stream.
		OutputStream->Logf(TEXT("%s"),*CurrentRow);

		// Reset the current row.
		CurrentRow.Empty();
	}

	/** Closes the output stream. */
	void Close()
	{
		if(OutputStream)
		{
			// Close the output stream.
			OutputStream->Close();
			OutputStream = NULL;
		}
	}

	/** Check for a valid output stream */
	bool OutputStreamIsValid()
	{
		return (OutputStream != NULL);
	}

private:

	/** The contents of the current row. */
	FString CurrentRow;

	/** The stream the table is being written to. */
	FArchive* OutputStream;
};

/** Encapsulates a diagnostic table that is written to a temporary file and opened in a viewer. */
class FDiagnosticTableViewer : public FDiagnosticTableWriterCSV
{
public:

	/** Creates the viewer with a uniquely named temporary file. */
	static FString GetUniqueTemporaryFilePath(const TCHAR* BaseName)
	{
		return FString::Printf(TEXT("%sLogs/%s-%s.csv"), *FPaths::ProjectDir(), BaseName, *FDateTime::Now().ToString());
	}

	/** Initialization constructor. */
	FDiagnosticTableViewer(const TCHAR* InTemporaryFilePath, bool bInSuppressViewer = false)
	:	FDiagnosticTableWriterCSV(IFileManager::Get().CreateDebugFileWriter(InTemporaryFilePath))
	,	bHasOpenedViewer(false)
	,	bSuppressViewer(bInSuppressViewer)
	,	TemporaryFilePath(InTemporaryFilePath)
	{}

	/** Destructor. */
	~FDiagnosticTableViewer()
	{
		OpenViewer();
	}

	/** Forces the writer to close and opens the written table in the viewer. */
	void OpenViewer()
	{
		if(FPlatformProperties::HasEditorOnlyData() && !bHasOpenedViewer && !bSuppressViewer)
		{
			bHasOpenedViewer = true;

			// Close the writer.
			Close();

			// Open the viewer.
			FPlatformProcess::LaunchURL(*FString::Printf(TEXT("%s"),*IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*TemporaryFilePath)), NULL, NULL);
		}
	}

private:
	
	/** True if the file has been opened in the viewer. */
	bool bHasOpenedViewer;

	bool bSuppressViewer;

	/** The filename used for the temporary file. */
	FString TemporaryFilePath;
};

#endif

