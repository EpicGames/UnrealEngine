// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringConv.h"
#include "CoreTypes.h"
#include "Features/IModularFeature.h"
#include "Misc/Build.h"
#include "ProfilingDebugging/ExternalProfilerConfig.h"
#include "UObject/NameTypes.h"

#if UE_EXTERNAL_PROFILING_ENABLED

/**
 * FExternalProfiler
 *
 * Interface to various external profiler API functions, dynamically linked
 */
class FExternalProfiler : public IModularFeature
{

public:

	/**
	 * Constructor. An external profiler being constructed must assume it starts out paused.
	 * To start profiling, call FActiveExternalProfilerBase::SetActiveProfilerRecording(true).
	 */
	CORE_API FExternalProfiler();

	/** Empty virtual destructor. */
	virtual ~FExternalProfiler() = default;

	/**
	 * Profiler interface.
	 */

	/** Mark where the profiler should consider the frame boundary to be. */
	virtual void FrameSync() = 0;

	/** Initialize  profiler, register some delegates..*/
	virtual void Register() {}

	/** Pauses profiling. */
	virtual void ProfilerPauseFunction() = 0;

	/** Resumes profiling. */
	virtual void ProfilerResumeFunction() = 0;

	/** Gets the name of this profiler as a string.  This is used to allow the user to select this profiler in a system configuration file or on the command-line */
	virtual const TCHAR* GetProfilerName() const = 0;

	/** @return Returns the name to use for any profiler registered as a modular feature usable by this system */
	static CORE_API FName GetFeatureName();

	/** Notification that scoped event transmission status changed. */
	virtual void OnEnableScopedEventsChanged(bool bEnabled) {};

	/** Starts a scoped event specific to the profiler. */
	virtual void StartScopedEvent(const struct FColor& Color, const TCHAR* Text) {};

	/** Starts a scoped event specific to the profiler. Default implementation for backward compabitility. */
	virtual void StartScopedEvent(const struct FColor& Color, const ANSICHAR* Text) { StartScopedEvent(Color, ANSI_TO_TCHAR(Text)); };

	/** Ends a scoped event specific to the profiler. */
	virtual void EndScopedEvent() {};

	virtual void SetThreadName(const TCHAR* Name) {}
};

class FActiveExternalProfilerBase
{
public:	

	static FExternalProfiler* GetActiveProfiler() { return ActiveProfiler;	};

	/**
	 * Attach an external profiler, selected depending on configuration.
	 *
	 * Normally, no external profiler is selected, but one can be set using one of the following:
	 *
	 * 1. A command line argument `-ProfilerName`, where `ProfilerName` matches the string returned
	 *    by `FExternalProfiler::GetProfilerName()`
	 * 2. The environment variable `UE_EXTERNAL_PROFILER=ProfilerName`, where `ProfilerName` matches
	 *    the string returned by `FExternalProfiler::GetProfilerName()`
	 * 3. The DefaultEngine.ini config variable `[Core.ProfilingDebugging]:ExternalProfiler`, where
	 *    the value is a string that matches the string returned by `FExternalProfiler::GetProfilerName()`.
	 *
	 * In that order of priority.
	 * If no profiler is found matching these criteria, no profiler (nullptr) is attached.
	 *
	 * To make a profiler discoverable by this function, register it into the `FExternalProfiler`
	 * modular feature.
	 *
	 * The search for a selected profiler is only performed once and then cached.
	 * On subsequent calls, the cached result is returned.
	 */
	static CORE_API FExternalProfiler* InitActiveProfiler();

	/**
	 * Pauses or resumes the active profiler.
	 * Returns the previous recording state (true if it was recording, false if it was paused.)
	 */
	static CORE_API bool SetActiveProfilerRecording(bool bRecording);
	static CORE_API bool IsActiveProfilerRecording();

	/**
	 * Static: Enables or disables calls to the active profiler's StartScopedEvent and EndScopedEvent functions.
	 *
	 * This function calls OnEnableScopedEventsChanged on the active external profiler if the enabled status
	 * has changed. 
	 *  
	 * External profilers may support initiating captures either from an external tool or via code.
	 * The performance cost of emitting scoped events varies by profiler. To minimize overhead, including 
	 * calls to stubbed virtual functions, EnableScopedEvents can be used by a profiler implementation to
	 * control event emission.
	 * 
	 * Example scenarios:
	 * 
	 * 1. A profiler that doesn't implement scoped events can disable scoped events to avoid the overhead
	 *    of calling stubbed virtual functions.
	 * 2. A profiler that implements scoped events and initiates captures externally can leverage 
	 *    the default behavior of emitting events.
	 * 3. A profiler with high overhead scoped events may disable scoped events to avoid overhead and 
	 *    require explicit user action to enable them.
	 * 4. A profiler that uses code-initiated captures may enable scoped events when capture starts
	 *    and disables events when the capture ends.
	 */
	static CORE_API void EnableScopedEvents(bool bEnable);

	/**
	 * Returns true if scoped events are enabled by the active profiler.
	 */
	static CORE_API bool AreScopedEventsEnabled();

private:
	/** Static: True if we've tried to initialize a profiler already */
	static CORE_API bool bDidInitialize;

	/** Static: Active profiler instance that we're using */
	static CORE_API FExternalProfiler* ActiveProfiler;

	/** Static: Whether the active profiler is recording events, to our knowledge. */
	static std::atomic<bool> bIsRecording;

	/** Static: Whether StartScopedEvent and EndScopedEvent should be called. Defaults to true. */
	static std::atomic<bool> bEnableScopedEvents;
};

/**
 * Base class for FScoped*Timer and FScoped*Excluder
 */
class FScopedExternalProfilerBase : public FActiveExternalProfilerBase
{
protected:
	/**
	 * Pauses or resumes profiler and keeps track of the prior state so it can be restored later.
	 *
	 * @param	bWantPause	true if this timer should 'include' code, or false to 'exclude' code
	 *
	 **/
	CORE_API void StartScopedTimer( const bool bWantPause );

	/** Stops the scoped timer and restores profiler to its prior state */
	CORE_API void StopScopedTimer();

private:
	/** Stores the previous 'Paused' state of the current active external profiler before this scope started */
	bool bWasRecording = false;
};


/**
 * Scoped guard that includes a body of code in the external profiler's captured session, by calling
 * `SetActiveProfilerRecording(true)` in its constructor, and then
 * `SetActiveProfilerRecording(false)` in its destructor.
 *
 * It can safely be embedded within another `FExternalProfilerIncluder` or `Excluder`. However,
 * `FActiveProfilerBase::SetActiveProfilerRecording` should not be called from within an `Includer`
 * scope, because the `Includer` and `Excluder` mechanisms cannot record changes to the active
 * profiler state once they're in scope---you should choose to use one mechanism or the other,
 * but not both.
 */
class FExternalProfilerIncluder : public FScopedExternalProfilerBase
{
public:
	/** Constructor */
	FExternalProfilerIncluder()
	{
		const bool bWantPause = false;
		StartScopedTimer( bWantPause );
	}

	/** Destructor */
	~FExternalProfilerIncluder()
	{
		StopScopedTimer();
	}
};


/**
 * Scoped guard that excludes a body of code in the external profiler's captured session, by calling
 * `SetActiveProfilerRecording(false)` in its constructor, and then
 * `SetActiveProfilerRecording(true)` in its destructor.
 *
 * It can safely be embedded within another `FExternalProfilerExcluder` or `Includer`. However,
 * `FActiveProfilerBase::SetActiveProfilerRecording` should not be called from within an `Includer`
 * scope, because the `Includer` and `Excluder` mechanisms cannot record changes to the active
 * profiler state once they're in scope---you should choose to use one mechanism or the other,
 * but not both.
 */
class FExternalProfilerExcluder : public FScopedExternalProfilerBase
{
public:
	/** Constructor */
	FExternalProfilerExcluder()
	{
		const bool bWantPause = true;
		StartScopedTimer( bWantPause );
	}

	/** Destructor */
	~FExternalProfilerExcluder()
	{
		StopScopedTimer();
	}

};

class FExternalProfilerTrace
{
public:
	/** Starts a scoped event specific to the profiler. */
	inline static void StartScopedEvent(const struct FColor& Color, const TCHAR* Text)
	{
		if (FActiveExternalProfilerBase::AreScopedEventsEnabled())
		{
			if (FExternalProfiler* Profiler = FActiveExternalProfilerBase::GetActiveProfiler())
			{
				Profiler->StartScopedEvent(Color, Text);
			}
		}
	}

	/** Starts a scoped event specific to the profiler. */
	inline static void StartScopedEvent(const struct FColor& Color, const ANSICHAR* Text)
	{
		if (FActiveExternalProfilerBase::AreScopedEventsEnabled())
		{
			if (FExternalProfiler* Profiler = FActiveExternalProfilerBase::GetActiveProfiler())
			{
				Profiler->StartScopedEvent(Color, Text);
			}
		}
	}

	/** Ends a scoped event specific to the profiler. */
	inline static void EndScopedEvent()
	{
		if (FActiveExternalProfilerBase::AreScopedEventsEnabled())
		{
			if (FExternalProfiler* Profiler = FActiveExternalProfilerBase::GetActiveProfiler())
			{
				Profiler->EndScopedEvent();
			}
		}
	}
};

#define SCOPE_PROFILER_INCLUDER(X) FExternalProfilerIncluder ExternalProfilerIncluder_##X;
#define SCOPE_PROFILER_EXCLUDER(X) FExternalProfilerExcluder ExternalProfilerExcluder_##X;

#else	// UE_EXTERNAL_PROFILING_ENABLED

#define SCOPE_PROFILER_INCLUDER(X)
#define SCOPE_PROFILER_EXCLUDER(X)

#endif	// !UE_EXTERNAL_PROFILING_ENABLED
