// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

#ifndef UE_EXTERNAL_PROFILING_ENABLED
#define UE_EXTERNAL_PROFILING_ENABLED (!UE_BUILD_SHIPPING)
#endif
