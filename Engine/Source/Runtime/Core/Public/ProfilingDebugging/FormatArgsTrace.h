// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreTypes.h>
#include "Containers/ContainerAllocationPolicies.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/CString.h"
#include "Templates/IsFloatingPoint.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "Traits/IsCharType.h"

struct FFormatArgsTrace
{
	enum EFormatArgTypeCode
	{
		FormatArgTypeCode_CategoryBitShift = 6,
		FormatArgTypeCode_SizeBitMask = (1 << FormatArgTypeCode_CategoryBitShift) - 1,
		FormatArgTypeCode_CategoryBitMask = ~FormatArgTypeCode_SizeBitMask,
		FormatArgTypeCode_CategoryInteger = 1 << FormatArgTypeCode_CategoryBitShift,
		FormatArgTypeCode_CategoryFloatingPoint = 2 << FormatArgTypeCode_CategoryBitShift,
		FormatArgTypeCode_CategoryString = 3 << FormatArgTypeCode_CategoryBitShift,
	};

	// Convenient alias to use when allowing overflow for very large messages
	template <int InlineSize>
	using TBufferWithOverflow = TArray<uint8, TInlineAllocator<InlineSize>>;

	// Encode arguments in any type of resizable array
	template <typename BufferType, typename... Types>
	static int32 EncodeArgumentsWithArray(BufferType& Array, Types... FormatArgs)
	{
		constexpr int32 FormatArgsCount = sizeof...(FormatArgs);
		static_assert(FormatArgsCount < 256, "Maximum number of arguments is 256");
		const int32 FormatArgsSize = 1 + FormatArgsCount + int32(GetArgumentsEncodedSize(FormatArgs...));
		Array.SetNumUninitialized(FormatArgsSize);
		uint8* Buffer = Array.GetData();
		uint8* TypeCodesBufferPtr = Buffer;
		*TypeCodesBufferPtr++ = uint8(FormatArgsCount);
		uint8* PayloadBufferPtr = TypeCodesBufferPtr + FormatArgsCount;
		EncodeArgumentsInternal(TypeCodesBufferPtr, PayloadBufferPtr, FormatArgs...);
		check(PayloadBufferPtr - Buffer == FormatArgsSize);
		return FormatArgsSize;
	}

	// Encode arguments in a fixed size buffer
	template <int BufferSize, typename... Types>
	static uint16 EncodeArguments(uint8(&Buffer)[BufferSize], Types... FormatArgs)
	{
		static_assert(BufferSize < 65536, "Maximum buffer size of 16 bits exceeded");
		uint64 FormatArgsCount = sizeof...(FormatArgs);
		if (FormatArgsCount >= 256)
		{
			// Nope
			return 0;
		}
		uint64 FormatArgsSize = 1 + FormatArgsCount + GetArgumentsEncodedSize(FormatArgs...);
		if (FormatArgsSize > BufferSize)
		{
			// Nope
			return 0;
		}
		uint8* TypeCodesBufferPtr = Buffer;
		*TypeCodesBufferPtr++ = uint8(FormatArgsCount);
		uint8* PayloadBufferPtr = TypeCodesBufferPtr + FormatArgsCount;
		EncodeArgumentsInternal(TypeCodesBufferPtr, PayloadBufferPtr, FormatArgs...);
		check(PayloadBufferPtr - Buffer == FormatArgsSize);
		return (uint16)FormatArgsSize;
	}

private:
	template <typename T>
	struct TIsStringArgument
	{
		enum { Value = TAnd<TIsPointer<T>, TIsCharType<std::remove_cv_t<typename TRemovePointer<T>::Type>>>::Value };
	};

	template <typename T, typename CharType = std::remove_cv_t<typename TRemovePointer<T>::Type>>
	constexpr static uint64 GetArgumentEncodedSize(T Argument)
	{
		if constexpr (TIsStringArgument<T>::Value)
		{
			if (Argument != nullptr)
			{
				return FMath::Min<int32>((int32)UINT16_MAX, (TCString<CharType>::Strlen(Argument) + 1) * sizeof(CharType));
			}
			else
			{
				return sizeof(CharType);
			}
		}
		else
		{
			return sizeof(T);
		}
	}

	constexpr static uint64 GetArgumentsEncodedSize()
	{
		return 0;
	}

	template <typename T, typename... Types>
	static uint64 GetArgumentsEncodedSize(T Head, Types... Tail)
	{
		return GetArgumentEncodedSize(Head) + GetArgumentsEncodedSize(Tail...);
	}

	template <typename T>
	static void EncodeArgumentInternal(uint8*& TypeCodesPtr, uint8*& PayloadPtr, T Argument)
	{
		if constexpr (TIsStringArgument<T>::Value)
		{
			using CharType = std::remove_cv_t<typename TRemovePointer<T>::Type>;

			*TypeCodesPtr++ = FormatArgTypeCode_CategoryString | sizeof(CharType);
			if (Argument != nullptr)
			{
				uint16 SizeBytes = (uint16)FMath::Min<int32>((int32)UINT16_MAX, (TCString<CharType>::Strlen(Argument) + 1) * sizeof(CharType));
				memcpy(PayloadPtr, Argument, SizeBytes);
				if (UNLIKELY(SizeBytes == UINT16_MAX))
				{
					const CharType Ellipsis[4] = {L'.', L'.', L'.', L'\0'};
					// Patch ellipsis to the end of the string. Make sure write is aligned
					void* PatchPoint = PayloadPtr + SizeBytes - sizeof(Ellipsis) - (SizeBytes % sizeof(CharType));
					memcpy(PatchPoint, Ellipsis, sizeof(Ellipsis));
				}
				PayloadPtr += SizeBytes;
			}
			else
			{
				CharType Terminator { 0 };
				memcpy(PayloadPtr, &Terminator, sizeof(CharType));
				PayloadPtr += sizeof(CharType);
			}
		}
		else
		{
			if constexpr (std::is_floating_point_v<T>)
			{
				*TypeCodesPtr++ = FormatArgTypeCode_CategoryFloatingPoint | sizeof(T);
			}
			else
			{
				*TypeCodesPtr++ = FormatArgTypeCode_CategoryInteger | sizeof(T);
			}

#if PLATFORM_SUPPORTS_UNALIGNED_LOADS
			*reinterpret_cast<T*>(PayloadPtr) = Argument;
#else
			// For ARM targets, it's possible that using __packed here would be preferable
			// but I have not checked the codegen -- it's possible that the compiler generates
			// the same code for this fixed size memcpy
			memcpy(PayloadPtr, &Argument, sizeof Argument);
#endif

			PayloadPtr += sizeof(T);
		}
	}

	constexpr static void EncodeArgumentsInternal(uint8*& TypeCodesPtr, uint8*& PayloadPtr)
	{
	}

	template <typename T, typename... Types>
	static void EncodeArgumentsInternal(uint8*& ArgDescriptorsPtr, uint8*& ArgPayloadPtr, T Head, Types... Tail)
	{
		EncodeArgumentInternal(ArgDescriptorsPtr, ArgPayloadPtr, Head);
		EncodeArgumentsInternal(ArgDescriptorsPtr, ArgPayloadPtr, Tail...);
	}
};