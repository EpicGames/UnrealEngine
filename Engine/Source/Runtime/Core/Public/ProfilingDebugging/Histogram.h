// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Here are a number of profiling helper functions so we do not have to duplicate a lot of the glue
 * code everywhere.  And we can have consistent naming for all our files.
 *
 */

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"

#include <initializer_list>

struct FAnalyticsEventAttribute;
struct FHistogramBuilder;
template <typename FuncType> class TFunctionRef;

DECLARE_LOG_CATEGORY_EXTERN(LogHistograms, Log, All);

 struct FHistogramBuilder;

 /** Fairly generic histogram for values that have natural lower bound and possibly no upper bound, e.g., frame time */
struct FHistogram
{
	/** Inits histogram with linear, equally sized bins */
	CORE_API void InitLinear(double MinTime, double MaxTime, double BinSize);

	/** Inits histogram to mimic our existing hitch buckets */
	CORE_API void InitHitchTracking();

	/** Inits histogram with the specified bin boundaries, with the final bucket extending to infinity (e.g., passing in 0,5 creates a [0..5) bucket and a [5..infinity) bucket) */
	CORE_API void InitFromArray(TArrayView<const double> Thresholds);

	/** Inits histogram with the specified bin boundaries, with the final bucket extending to infinity (e.g., passing in 0,5 creates a [0..5) bucket and a [5..infinity) bucket) */
	UE_FORCEINLINE_HINT void InitFromArray(std::initializer_list<double> Thresholds)
	{
		InitFromArray(MakeArrayView(Thresholds));
	}

	/** Resets measurements, without resetting the configured bins. */
	CORE_API void Reset();

	/** Adds an observed measurement. */
	inline void AddMeasurement(double Value)
	{
		AddMeasurement(Value, Value);
	}

	/** Adds an observed measurement (with a different thresholding key than the measurement, e.g., when accumulating time spent in a chart keyed on framerate). */
	CORE_API void AddMeasurement(double ValueForBinning, double MeasurementValue);

	/** Prints histogram contents to the log. */
	CORE_API void DumpToLog(const FString& HistogramName);

	/** 
	 * Returns a string in a Json format: [{"Bin":"BinName","Count":Count,"Sum":Sum},...]. 
	 * Bucket name is constructed by calling ConvertBinToLabel on the MinValue and UpperBound for each bin. 
	 * Convert function is used to allow the bin range, which is stored as a double, to be printed prettily.
	 */
	CORE_API FString DumpToJsonString(TFunctionRef<FString (double, double)> ConvertBinToLabel) const;

	/** Same as DumpToJsonString but uses a DefaultConvertBinToLabel. */
	CORE_API FString DumpToJsonString() const;

	/** 
	* Returns a string in a Json format: [{"BinName":{"Count":Count,"Sum":Sum}},...]. 
	* Bucket name is constructed by calling ConvertBinToLabel on the MinValue and UpperBound for each bin. 
	* Convert function is used to allow the bin range, which is stored as a double, to be printed prettily.
	*/
	CORE_API FString DumpToJsonString2(TFunctionRef<FString (double, double)> ConvertBinToLabel) const;

	/** Same as DumpToJsonString2 but uses a DefaultConvertBinToLabel. */
	CORE_API FString DumpToJsonString2() const;

	/** Default stringifier for bins for use with DumpToJsonString. Truncates to int and uses Plus as the suffix for the last bin. ie. [0.0, 3.75, 9.8] -> 0_3, 3_9, 9_Plus */
	static CORE_API FString DefaultConvertBinToLabel(double MinValue, double UpperBound);

	/** Gets number of bins. */
	inline int32 GetNumBins() const
	{
		return Bins.Num();
	}

	/** Gets lower bound of the bin, i.e. minimum value that goes into it. */
	inline double GetBinLowerBound(int IdxBin) const
	{
		return Bins[IdxBin].MinValue;
	}

	/** Gets upper bound of the bin, i.e. first value that does not go into it. */
	inline double GetBinUpperBound(int IdxBin) const
	{
		return Bins[IdxBin].UpperBound;
	}

	/** Gets number of observations in the bin. */
	inline int32 GetBinObservationsCount(int IdxBin) const
	{
		return Bins[IdxBin].Count;
	}

	/** Gets sum of observations in the bin. */
	inline double GetBinObservationsSum(int IdxBin) const
	{
		return Bins[IdxBin].Sum;
	}

	/** Returns the sum of all counts (the number of recorded measurements) */
	inline int64 GetNumMeasurements() const
	{
		return CountOfAllMeasures;
	}

	/** Returns the sum of all measurements */
	inline double GetSumOfAllMeasures() const
	{
		return SumOfAllMeasures;
	}

	/** Returns the average of all measurements (essentially a shortcut for Sum/Count). */
	inline double GetAverageOfAllMeasures() const
	{
		return (CountOfAllMeasures > 0) ? (SumOfAllMeasures / (double)CountOfAllMeasures) : 0 ;
	}

	/** Returns the minimum of all measurements. */
	inline double GetMinOfAllMeasures() const
	{
		return MinimalMeasurement;
	}

	/** Returns the maximum of all measurements. */
	inline double GetMaxOfAllMeasures() const
	{
		return MaximalMeasurement;
	}

	inline FHistogram operator-(const FHistogram& Other) const
	{
		// Logic below expects the bins to be of equal size
		check(GetNumBins() == Other.GetNumBins());

		FHistogram Tmp;
		for (int32 BinIndex = 0, NumBins = Bins.Num(); BinIndex < NumBins; BinIndex++)
		{
			Tmp.Bins.Add(Bins[BinIndex] - Other.Bins[BinIndex]);
		}
		return Tmp;
	}

	inline FHistogram operator+(const FHistogram& Other) const
	{
		// Logic below expects the bins to be of equal size
		check(GetNumBins() == Other.GetNumBins());

		FHistogram Tmp;
		for (int32 BinIndex = 0, NumBins = Bins.Num(); BinIndex < NumBins; BinIndex++)
		{
			Tmp.Bins.Add(Bins[BinIndex] + Other.Bins[BinIndex]);
		}
		return Tmp;
	}

	inline FHistogram& operator+=(const FHistogram& Other)
	{
		// Logic below expects the bins to be of equal size
		check(GetNumBins() == Other.GetNumBins());

		for (int32 BinIndex = 0, NumBins = Bins.Num(); BinIndex < NumBins; BinIndex++)
		{
			Bins[BinIndex] += Other.Bins[BinIndex];
		}

		SumOfAllMeasures += Other.SumOfAllMeasures;
		CountOfAllMeasures += Other.CountOfAllMeasures;
		MinimalMeasurement = FMath::Min(MinimalMeasurement, Other.MinimalMeasurement);
		MaximalMeasurement = FMath::Max(MaximalMeasurement, Other.MaximalMeasurement);

		return *this;
	}

protected:

	/** Bin */
	struct FBin
	{
		/** MinValue to be stored in the bin, inclusive. */
		double MinValue;

		/** First value NOT to be stored in the bin. */
		double UpperBound;

		/** Sum of all values that were put into this bin. */
		double Sum;

		/** How many elements are in this bin. */
		int32 Count;

		FBin()
		{
		}

		/** Constructor for a pre-seeded bin */
		FBin(double MinInclusive, double MaxExclusive, double InSum, int32 InCount)  //@TODO: FLOATPRECISION: THIS CHANGES BEHAVIOR
			: MinValue(MinInclusive)
			, UpperBound(MaxExclusive)
			, Sum(InSum)
			, Count(InCount)
		{
		}

		/** Constructor for any bin */
		FBin(double MinInclusive, double MaxExclusive)
			: MinValue(MinInclusive)
			, UpperBound(MaxExclusive)
			, Sum(0)
			, Count(0)
		{
		}

		/** Constructor for the last bin. */
		FBin(double MinInclusive)
			: MinValue(MinInclusive)
			, UpperBound(FLT_MAX)
			, Sum(0)
			, Count(0)
		{
		}

		UE_FORCEINLINE_HINT FBin operator-(const FBin& Other) const
		{
			return FBin(MinValue, UpperBound, Sum - Other.Sum, Count - Other.Count);
		}
		UE_FORCEINLINE_HINT FBin operator+(const FBin& Other) const
		{
			return FBin(MinValue, UpperBound, Sum + Other.Sum, Count + Other.Count);
		}
		inline FBin& operator+=(const FBin& Other)
		{
			Sum += Other.Sum;
			Count += Other.Count;
			return *this;
		}
	};

	/** Bins themselves, should be continous in terms of [MinValue; UpperBound) and sorted ascending by MinValue. Last bin's UpperBound doesn't matter */
	TArray<FBin> Bins;

	/** Quick stats for all bins */
	double SumOfAllMeasures;
	int64 CountOfAllMeasures;
	double MinimalMeasurement;
	double MaximalMeasurement;

	/** This is exposed as a clean way to build bins while enforcing the condition mentioned above */
	friend FHistogramBuilder;
};

/** Used to construct a histogram that runs over a custom set of ranges, while still enforcing contiguity on the bin ranges */
struct FHistogramBuilder
{
	FHistogramBuilder(FHistogram& InHistogram, double StartingValue = 0.0)
		: MyHistogram(&InHistogram)
		, LastValue(StartingValue)
	{
		MyHistogram->SumOfAllMeasures = 0.0;
		MyHistogram->CountOfAllMeasures = 0;
		MyHistogram->MinimalMeasurement = DBL_MAX;
		MyHistogram->MaximalMeasurement = -DBL_MAX;
		MyHistogram->Bins.Reset();
	}

	/** This will add a bin to the histogram, extending from the previous bin (or starting value) to the passed in MaxValue */
	void AddBin(double MaxValue)
	{
		// Can't add to a closed builder
		check(MyHistogram);
		MyHistogram->Bins.Emplace(LastValue, MaxValue);
		LastValue = MaxValue;
	}

	/** Call when done adding bins, this will create a final unbounded bin to catch values above the maximum value */
	void FinishBins()
	{
		MyHistogram->Bins.Emplace(LastValue);
		MyHistogram = nullptr;
	}

	~FHistogramBuilder()
	{
		// Close it out if the user hasn't already
		if (MyHistogram != nullptr)
		{
			FinishBins();
		}
	}
private:
	FHistogram* MyHistogram;
	double LastValue;
};

