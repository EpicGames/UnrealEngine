// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Build.h"
#include "ProfilingDebugging/MetadataTrace.h"
#include "Trace/Config.h"

#define UE_API CORE_API

////////////////////////////////////////////////////////////////////////////////////////////////////
#if !defined(UE_TRACE_HTTP_ENABLED) // to allow users to manually disable it from build.cs files
	#if !UE_BUILD_SHIPPING
		#define UE_TRACE_HTTP_ENABLED UE_TRACE_ENABLED
	#else
		#define UE_TRACE_HTTP_ENABLED 0
	#endif
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////
#if !defined(UE_TRACE_HTTP_METADATA_ENABLED)
	#define UE_TRACE_HTTP_METADATA_ENABLED UE_TRACE_HTTP_ENABLED && UE_TRACE_METADATA_ENABLED
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_HTTP_ENABLED

UE_TRACE_CHANNEL_EXTERN(HttpChannel, CORE_API);

////////////////////////////////////////////////////////////////////////////////////////////////////
struct FHttpTrace
{
	static UE_API void DispatcherCreated(uint64 Dispatcher, const ANSICHAR* Name);
	static UE_API void CategoryCreated(uint8 Category, const ANSICHAR* Name);
	static UE_API void RequestStarted(uint64 Dispatcher, uint64 Request, const ANSICHAR* Url, int32 Priority, uint8 Category);
	static UE_API void ChunkRangeAdded(uint64 Request, const class FIoChunkId& ChunkId, uint32 Start, uint32 End);
	static UE_API void RequestCompleted(uint64 Request, const ANSICHAR* Host, uint32 StatusCode, uint32 ContentLength);
};

////////////////////////////////////////////////////////////////////////////////////////////////////
#define TRACE_HTTP_DISPATCHER_CREATED(Dispatcher, Name) \
	FHttpTrace::DispatcherCreated(Dispatcher, Name);

#define TRACE_HTTP_CATEGORY_CREATED(Category, Name) \
	FHttpTrace::CategoryCreated(Category, Name);

#define TRACE_HTTP_REQUEST_STARTED(Dispatcher, Request, Url, Priority, Category) \
	FHttpTrace::RequestStarted(Dispatcher, Request, Url, Priority, Category);

#define TRACE_HTTP_CHUNK_RANGE_ADDED(Request, ChunkId, Start, End) \
	FHttpTrace::ChunkRangeAdded(Request, ChunkId, Start, End);

#define TRACE_HTTP_REQUEST_COMPLETED(Request, Host, StatusCode, ContentLength) \
	FHttpTrace::RequestCompleted(Request, Host, StatusCode, ContentLength);

////////////////////////////////////////////////////////////////////////////////////////////////////
#else

#define TRACE_HTTP_DISPATCHER_CREATED(...)
#define TRACE_HTTP_CATEGORY_CREATED(Category, Name)(...)
#define TRACE_HTTP_REQUEST_STARTED(...)
#define TRACE_HTTP_CHUNK_RANGE_ADDED(Request, ChunkId, Start, End)(...)
#define TRACE_HTTP_REQUEST_COMPLETED(...)

#endif //UE_TRACE_HTTP_ENABLED

#undef UE_API
