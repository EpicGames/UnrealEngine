// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/PreprocessorHelpers.h"
#include "Misc/Build.h"
#include "UObject/NameTypes.h"

class FOutputDevice;

/*
	A helper object for counting instances of struct or classes for debugging
	purposes, e.g. to get an absolute count of the number of "Foo"'s in existence

	Suggested use

	class UFoo
	{
	public:
		// etc etc

	protected:
		COUNT_INSTANCES(FTypeIWantToCount)
	};

*/
class FInstanceCountingObject
{
public:

	/**
	 *	Constructor, though geneerally these objects should be created using the COUNT_INSTANCES_ 
	 *	macro.
	 */
	CORE_API FInstanceCountingObject(const TCHAR* InName, bool InLogConstruction=false);

	/**
	 *	Copy-constructor for assigment
	 */
	CORE_API FInstanceCountingObject(const FInstanceCountingObject& RHS);

	/**	Destructor */
	CORE_API virtual ~FInstanceCountingObject();

	/**
	 * Returns the count of instances with "Name". 
	 *
	 * @param: 	Name		
	 * @return: int32		
	*/static CORE_API int32 GetInstanceCount(const TCHAR* Name);
	
	/**
	 * Dumps stats for all counted instances to the provided output device. This
	 * is bound to the "LogCountedInstances" console command
	 *
	 * @param: 	OutputDevice		
	 * @return: void		
	*/static CORE_API void LogCounts(FOutputDevice& OutputDevice);
	
protected:

	/**	Increments stats for objects of this type */
	CORE_API void IncrementStats();

	/**	Decrements stats for objects of this type */
	CORE_API void DecrementStats();

protected:
	
	/**	Name we are tracking */
	FName		Name;

	// Log increment/decrement?
	bool		 DoLog;

	/**
	 *	Vars used by our singleton
	 */
	struct FGlobalVars
	{
		TMap<FName, int32>	InstanceCounts;
		FCriticalSection	Mutex;
	};

	/**
	 *	Vars are stored as a pointer and initialized on demand due to avoid dependencies
	 *	on global crot order
	 */
	static CORE_API FGlobalVars* Globals;
	static CORE_API FGlobalVars& GetGlobals();
};


#if !UE_BUILD_SHIPPING

	/**
	 *	Add to class body of TypeName to track counts of "TypeName". Counts can be accessed via code or console
	 */
	#define COUNT_INSTANCES(TypeName) \
		struct FCount##TypeName : FInstanceCountingObject { FCount##TypeName() : FInstanceCountingObject(TEXT(UE_STRINGIZE(TypeName))) {} }; FCount##TypeName TypeName##Count; 

	 /**
	 *	Identical to COUNT_INSTANCES, but construction/destruction will be written to the log along with the address of this object (us, not the instance that contains us)
	 */
	#define COUNT_INSTANCES_AND_LOG(TypeName) \
		struct FCount##TypeName : FInstanceCountingObject { FCount##TypeName() : FInstanceCountingObject(TEXT(UE_STRINGIZE(TypeName)), true) {} }; FCount##TypeName TypeName##Count;


#else

	#define COUNT_INSTANCES(TypeName)
	#define COUNT_INSTANCES_AND_LOG(TypeName)

#endif // SHIPPING
