// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Build.h"
#include "Trace/Config.h"

// requires -trace=metadata,assetmetadata,iostore

#if !defined(UE_TRACE_IOSTORE_ENABLED) // to allow users to manually disable it from build.cs files
	#if !UE_BUILD_SHIPPING
		#define UE_TRACE_IOSTORE_ENABLED UE_TRACE_ENABLED
	#else
		#define UE_TRACE_IOSTORE_ENABLED 0
	#endif
#endif

#if UE_TRACE_IOSTORE_ENABLED


#include "ProfilingDebugging/MetadataTrace.h"
#include "ProfilingDebugging/StringsTrace.h"

class FIoBatch;
class FIoRequestImpl;
struct IIoDispatcherBackend;

struct FIoStoreTrace
{
	static CORE_API void BackendName(IIoDispatcherBackend* IoDispatcherBackend, const TCHAR* Name);
	static CORE_API void RequestCreate(FIoBatch* IoBatch, FIoRequestImpl* IoRequestImpl);
	static CORE_API void RequestUnresolved(FIoRequestImpl* IoRequestImpl);
	static CORE_API void RequestStarted(FIoRequestImpl* IoRequestImpl, IIoDispatcherBackend* IoDispatcherBackend);
	static CORE_API void RequestCompleted(FIoRequestImpl* IoRequestImpl, uint64 Size);
	static CORE_API void RequestFailed(FIoRequestImpl* IoRequestImpl);
};

UE_TRACE_CHANNEL_EXTERN(IoStoreChannel, CORE_API);

#define TRACE_IOSTORE_BACKEND_NAME(IoDispatcherBackend, Name) \
	FIoStoreTrace::BackendName(IoDispatcherBackend, Name);

#define TRACE_IOSTORE_REQUEST_CREATE(IoBatch,IoRequestImpl) \
	FIoStoreTrace::RequestCreate(IoBatch,IoRequestImpl);

#define TRACE_IOSTORE_REQUEST_UNRESOLVED(IoRequestImpl) \
	FIoStoreTrace::RequestUnresolved(IoRequestImpl);

#define TRACE_IOSTORE_BACKEND_REQUEST_STARTED(IoRequestImpl, IoDispatcherBackend) \
	FIoStoreTrace::RequestStarted(IoRequestImpl, IoDispatcherBackend);

#define TRACE_IOSTORE_BACKEND_REQUEST_COMPLETED(IoRequestImpl, Size) \
	FIoStoreTrace::RequestCompleted(IoRequestImpl, Size);

#define TRACE_IOSTORE_BACKEND_REQUEST_FAILED(IoRequestImpl) \
	FIoStoreTrace::RequestFailed(IoRequestImpl);

// creates a metadata scope that includes a tag string for identifying upackage data - used for IoStore usage where an asset/package cannot be determined (e.g. bulk data)
#if UE_TRACE_METADATA_ENABLED
	CORE_API UE_TRACE_METADATA_EVENT_BEGIN_EXTERN(IoStoreTag)
		UE_TRACE_METADATA_EVENT_REFERENCE_FIELD(Strings, FName, Tag)
	UE_TRACE_METADATA_EVENT_END()

	#define TRACE_IOSTORE_METADATA_SCOPE_TAG(TagName) \
		auto TagNameRef = bool(MetadataChannel) && bool(IoStoreChannel) ? FStringTrace::GetNameRef(TagName) : UE::Trace::FEventRef32(0,0); \
		UE_TRACE_METADATA_SCOPE(IoStoreTag, IoStoreChannel) \
			<< IoStoreTag.Tag(TagNameRef);
#else
	#define TRACE_IOSTORE_METADATA_SCOPE_TAG(...)
#endif

#else

#define TRACE_IOSTORE_BACKEND_NAME(...)
#define TRACE_IOSTORE_REQUEST_CREATE(...)
#define TRACE_IOSTORE_REQUEST_UNRESOLVED(...)
#define TRACE_IOSTORE_BACKEND_REQUEST_STARTED(...)
#define TRACE_IOSTORE_BACKEND_REQUEST_COMPLETED(...)
#define TRACE_IOSTORE_BACKEND_REQUEST_FAILED(...)

#define TRACE_IOSTORE_METADATA_SCOPE_TAG(...)

#endif //UE_TRACE_IOSTORE_ENABLED

