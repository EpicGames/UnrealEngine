// Copyright Epic Games, Inc. All Rights Reserved.

/**
* Declarations for LoadTimer which helps get load times for various parts of the game.
*/

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "CoreTypes.h"
#include "Misc/Build.h"
#include "Misc/Optional.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Serialization/LoadTimeTrace.h"
#include "Stats/Stats.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"
#include "UObject/NameTypes.h"

#ifndef ENABLE_LOADTIME_TRACKING
	#define ENABLE_LOADTIME_TRACKING 0
#endif

#ifndef ENABLE_LOADTIME_TRACKING_WITH_STATS
	#define ENABLE_LOADTIME_TRACKING_WITH_STATS 0
#endif

#define ENABLE_LOADTIME_RAW_TIMINGS 0

/** High level load time tracker utility (such as initial engine startup or game specific timings) */
class FLoadTimeTracker
{
public:
	static CORE_API FLoadTimeTracker& Get();

	/** Adds a scoped time for a given label.  Records each instance individually */
	CORE_API void ReportScopeTime(double ScopeTime, const FName ScopeLabel);

	/** Gets/adds a scoped time for a given label and instance. Records each instance individually */
	CORE_API double& GetScopeTimeAccumulator(const FName& ScopeLabel, const FName& ScopeInstance);

	/** Prints out total time and individual times */
	CORE_API void DumpHighLevelLoadTimes() const;

	static void DumpHighLevelLoadTimesStatic()
	{
		Get().DumpHighLevelLoadTimes();
	}

	const TMap<FName, TArray<double>>& GetData() const
	{
		return TimeInfo;
	}

	CORE_API void ResetHighLevelLoadTimes();

	/** Prints out raw load times for individual timers */
	CORE_API void DumpRawLoadTimes() const;

	static void DumpRawLoadTimesStatic()
	{
		Get().DumpRawLoadTimes();
	}

	CORE_API void ResetRawLoadTimes();

	static void ResetRawLoadTimesStatic()
	{
		Get().ResetRawLoadTimes();
	}

	CORE_API void StartAccumulatedLoadTimes();

	static void StartAccumulatedLoadTimesStatic()
	{
		Get().StartAccumulatedLoadTimes();
	}

	CORE_API void StopAccumulatedLoadTimes();

	static void StopAccumulatedLoadTimesStatic()
	{
		Get().StopAccumulatedLoadTimes();
	}

	bool IsAccumulating() { return bAccumulating; }

#if ENABLE_LOADTIME_RAW_TIMINGS

	/** Raw Timers */
	double CreateAsyncPackagesFromQueueTime;
	double ProcessAsyncLoadingTime;
	double ProcessLoadedPackagesTime;
	double SerializeTaggedPropertiesTime;
	double CreateLinkerTime;
	double FinishLinkerTime;
	double CreateImportsTime;
	double CreateExportsTime;
	double PreLoadObjectsTime;
	double PostLoadObjectsTime;
	double PostLoadDeferredObjectsTime;
	double FinishObjectsTime;
	double MaterialPostLoad;
	double MaterialInstancePostLoad;
	double SerializeInlineShaderMaps;
	double MaterialSerializeTime;
	double MaterialInstanceSerializeTime;
	double AsyncLoadingTime;
	double CreateMetaDataTime;

	double LinkerLoad_CreateLoader;
	double LinkerLoad_SerializePackageFileSummary;
	double LinkerLoad_SerializeNameMap;
	double LinkerLoad_SerializeGatherableTextDataMap;
	double LinkerLoad_SerializeImportMap;
	double LinkerLoad_SerializeExportMap;
	double LinkerLoad_FixupImportMap;
	double LinkerLoad_FixupExportMap;
	double LinkerLoad_SerializeMetaData;
	double LinkerLoad_SerializeDependsMap;
	double LinkerLoad_SerializePreloadDependencies;
	double LinkerLoad_CreateExportHash;
	double LinkerLoad_FindExistingExports;
	double LinkerLoad_FinalizeCreation;

	double Package_FinishLinker;
	double Package_LoadImports;
	double Package_CreateImports;
	double Package_CreateLinker;
	double Package_CreateExports;
	double Package_PreLoadObjects;
	double Package_ExternalReadDependencies;
	double Package_PostLoadObjects;
	double Package_Tick;
	double Package_CreateAsyncPackagesFromQueue;
	double Package_CreateMetaData;
	double Package_EventIOWait;

	double Package_Temp1;
	double Package_Temp2;
	double Package_Temp3;
	double Package_Temp4;

	double Graph_AddNode;
	uint32 Graph_AddNodeCnt;

	double Graph_AddArc;
	uint32 Graph_AddArcCnt;

	double Graph_RemoveNode;
	uint32 Graph_RemoveNodeCnt;

	double Graph_RemoveNodeFire;
	uint32 Graph_RemoveNodeFireCnt;

	double Graph_DoneAddingPrerequistesFireIfNone;
	uint32 Graph_DoneAddingPrerequistesFireIfNoneCnt;

	double Graph_DoneAddingPrerequistesFireIfNoneFire;
	uint32 Graph_DoneAddingPrerequistesFireIfNoneFireCnt;

	double Graph_Misc;
	uint32 Graph_MiscCnt;

	double TickAsyncLoading_ProcessLoadedPackages;

	double LinkerLoad_SerializeNameMap_ProcessingEntries;

	double FFileCacheHandle_AcquireSlotAndReadLine;
	double FFileCacheHandle_PreloadData;
	double FFileCacheHandle_ReadData;

	double FTypeLayoutDesc_Find;
	
	double FMemoryImageResult_ApplyPatchesFromArchive;
	double LoadImports_Event;
	double StartPrecacheRequests;
	double MakeNextPrecacheRequestCurrent;
	double FlushPrecacheBuffer;
	double ProcessImportsAndExports_Event;
	double CreateLinker_CreatePackage;
	double CreateLinker_SetFlags;
	double CreateLinker_FindLinker;
	double CreateLinker_GetRedirectedName;
	double CreateLinker_MassagePath;
	double CreateLinker_DoesExist;
	double CreateLinker_MissingPackage;
	double CreateLinker_CreateLinkerAsync;
	double FPackageName_DoesPackageExist;
	double PreLoadAndSerialize;
	double PostLoad;
	double LinkerLoad_ReconstructImportAndExportMap;
	double LinkerLoad_PopulateInstancingContext;
	double LinkerLoad_VerifyImportInner;
	double LinkerLoad_LoadAllObjects;
	double UObject_Serialize;
	double BulkData_Serialize;
	double BulkData_SerializeBulkData;
	double EndLoad;
	double FTextureReference_InitRHI;
	double FShaderMapPointerTable_LoadFromArchive;
	double FShaderLibraryInstance_PreloadShaderMap;
	double LoadShaderResource_Internal;
	double LoadShaderResource_AddOrDeleteResource;
	double FShaderCodeLibrary_LoadResource;
	double FMaterialShaderMapId_Serialize;
	double FMaterialShaderMapLayoutCache_CreateLayout;
	double FMaterialShaderMap_IsComplete;
	double FMaterialShaderMap_Serialize;
	double FMaterialResourceProxyReader_Initialize;
	double FSkeletalMeshVertexClothBuffer_InitRHI;
	double FSkinWeightVertexBuffer_InitRHI;
	double FStaticMeshVertexBuffer_InitRHI;
	double FStreamableTextureResource_InitRHI;
	double FShaderLibraryInstance_PreloadShader;
	double FShaderMapResource_SharedCode_InitRHI;
	double FStaticMeshInstanceBuffer_InitRHI;
	double FInstancedStaticMeshVertexFactory_InitRHI;
	double FLocalVertexFactory_InitRHI;
	double FLocalVertexFactory_InitRHI_CreateLocalVFUniformBuffer;
	double FSinglePrimitiveStructuredBuffer_InitRHI;
	double FColorVertexBuffer_InitRHI;
	double FFMorphTargetVertexInfoBuffers_InitRHI;
	double FSlateTexture2DRHIRef_InitDynamicRHI;
	double FLightmapResourceCluster_InitRHI;
	double UMaterialExpression_Serialize;
	double UMaterialExpression_PostLoad;
	double FSlateTextureRenderTarget2DResource_InitDynamicRHI;
	double VerifyGlobalShaders;
	double FLandscapeVertexBuffer_InitRHI;



#endif

private:
	TMap<FName, TArray<double>> TimeInfo;

	/** Track a time and count for a stat */
	struct FTimeAndCount
	{
		double Time;
		uint64 Count;
	};

	/** An accumulated stat group, with time and count for each instance */
	struct FAccumulatorTracker
	{
		TMap<FName, FTimeAndCount> TimeInfo;
	};

	/** Map to track accumulated timings */
	TMap<FName, FAccumulatorTracker> AccumulatedTimeInfo;

	/** We dont normally track accumulated load time info, only when this flag is true */
	bool bAccumulating;
private:
	CORE_API FLoadTimeTracker();
};

/** Scoped helper class for tracking accumulated object times */
struct FScopedLoadTimeAccumulatorTimer : public FScopedDurationTimer
{
	static CORE_API double DummyTimer;

	CORE_API FScopedLoadTimeAccumulatorTimer(const FName& InTimerName, const FName& InInstanceName);
};

#if ENABLE_LOADTIME_TRACKING
#define ACCUM_LOADTIME(TimerName, Time) FLoadTimeTracker::Get().ReportScopeTime(Time, FName(TimerName));
#else
#define ACCUM_LOADTIME(TimerName, Time)
#endif

#if ENABLE_LOADTIME_TRACKING
#define SCOPED_ACCUM_LOADTIME(TimerName, InstanceName) FScopedLoadTimeAccumulatorTimer AccumulatorTimer_##TimerName(FName(#TimerName), FName(InstanceName));
#else
#define SCOPED_ACCUM_LOADTIME(TimerName, InstanceName)
#endif

// Uses raw timers to store cumulative load times, does not support specific strings
#if ENABLE_LOADTIME_RAW_TIMINGS
#define SCOPED_LOADTIMER_TEXT(TimerName)
#define SCOPED_LOADTIMER_ASSET_TEXT(TimerName)
#define SCOPED_LOADTIMER(TimerName) FScopedDurationTimer DurationTimer_##TimerName(FLoadTimeTracker::Get().TimerName);
#define SCOPED_CUSTOM_LOADTIMER(TimerName)
#define SCOPED_LOADTIMER_CNT(TimerName) FScopedDurationTimer DurationTimer_##TimerName(FLoadTimeTracker::Get().TimerName); FLoadTimeTracker::Get().TimerName##Cnt++;
#define ADD_CUSTOM_LOADTIMER_META(TimerName, Key, Value)
#else

#define CUSTOM_LOADTIMER_LOG Cpu

// Uses trace system that can be read by Insights
#if LOADTIMEPROFILERTRACE_ENABLED

// Writes any string to the LoadTime channel, normally used for class names 
#define SCOPED_LOADTIMER_TEXT(TimerName) TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(TimerName, LoadTimeChannel)

// Writes any string to the AssetLoadTime channel, for full asset paths
#define SCOPED_LOADTIMER_ASSET_TEXT(TimerName) TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(TimerName, AssetLoadTimeChannel)

// Writes raw scope name to LoadTime channel
#define SCOPED_LOADTIMER(TimerName) TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(TimerName, LoadTimeChannel)

// Used to create a custom trace event and add metadata
#define SCOPED_CUSTOM_LOADTIMER(TimerName) UE_TRACE_LOG_SCOPED_T(CUSTOM_LOADTIMER_LOG, TimerName, LoadTimeChannel)
#define ADD_CUSTOM_LOADTIMER_META(TimerName, Key, Value) << TimerName.Key(Value)

// Increment cumulative event count, disabled in this mode
#define SCOPED_LOADTIMER_CNT(TimerName)

// All load time tracking is disabled
#else
#define SCOPED_LOADTIMER_TEXT(TimerName)
#define SCOPED_LOADTIMER_ASSET_TEXT(TimerName)
#define SCOPED_LOADTIMER(TimerName)
#define SCOPED_CUSTOM_LOADTIMER(TimerName)
#define ADD_CUSTOM_LOADTIMER_META(TimerName, Key, Value)
#define SCOPED_LOADTIMER_CNT(TimerName)
#endif
#endif

#if ENABLE_LOADTIME_TRACKING_WITH_STATS && STATS
	#define SCOPED_ACCUM_LOADTIME_STAT(InstanceName) FSimpleScopeSecondsStat ScopeTimer(FDynamicStats::CreateStatIdDouble<FStatGroup_STATGROUP_LoadTimeClass>(InstanceName, true), 1000.0);
	#define ACCUM_LOADTIMECOUNT_STAT(InstanceName) INC_DWORD_STAT_FNAME_BY(FDynamicStats::CreateStatIdInt64<FStatGroup_STATGROUP_LoadTimeClassCount>(InstanceName+TEXT("_Count"), true).GetName(), 1);
#else
	#define SCOPED_ACCUM_LOADTIME_STAT(...)
	#define ACCUM_LOADTIMECOUNT_STAT(...)
#endif
