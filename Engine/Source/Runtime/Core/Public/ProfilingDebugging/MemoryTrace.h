// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"
#include "Trace/Config.h"
#include "Trace/Trace.h"

#if !defined(UE_MEMORY_TRACE_AVAILABLE)
#	define UE_MEMORY_TRACE_AVAILABLE 0
#endif

#if !defined(UE_MEMORY_TRACE_LATE_INIT)
#	define UE_MEMORY_TRACE_LATE_INIT 0
#endif

#if !defined(UE_MEMORY_TRACE_ENABLED) && UE_TRACE_ENABLED
#	if UE_MEMORY_TRACE_AVAILABLE
#		if !PLATFORM_USES_FIXED_GMalloc_CLASS
#			define UE_MEMORY_TRACE_ENABLED !UE_BUILD_SHIPPING
#		endif
#	endif
#endif

#if !defined(UE_MEMORY_TRACE_ENABLED)
#	define UE_MEMORY_TRACE_ENABLED 0
#endif

////////////////////////////////////////////////////////////////////////////////
typedef uint32 HeapId;

////////////////////////////////////////////////////////////////////////////////
enum EMemoryTraceRootHeap : uint8
{
	SystemMemory, // RAM
	VideoMemory, // VRAM
	EndHardcoded = VideoMemory,
	EndReserved = 15
};

////////////////////////////////////////////////////////////////////////////////
// These values are traced. Do not modify existing values in order to maintain
// compatibility.
enum class EMemoryTraceHeapFlags : uint16
{
	None = 0,
	Root = 1 << 0,
	NeverFrees = 1 << 1, // The heap doesn't free (e.g. linear allocator)
};
ENUM_CLASS_FLAGS(EMemoryTraceHeapFlags);

////////////////////////////////////////////////////////////////////////////////
// These values are traced. Do not modify existing values in order to maintain
// compatibility.
enum class EMemoryTraceHeapAllocationFlags : uint8
{
	None = 0,
	Heap = 1 << 0, // Is a heap, can be used to unmark alloc as heap.
	Swap = 2 << 0, // Is a swap page
};
ENUM_CLASS_FLAGS(EMemoryTraceHeapAllocationFlags);

////////////////////////////////////////////////////////////////////////////////
enum class EMemoryTraceSwapOperation : uint8
{
	PageOut    = 0, // Paged out to swap
	PageIn     = 1, // Read from swap via page fault
	FreeInSwap = 2, // Freed while being paged out in swap
};

////////////////////////////////////////////////////////////////////////////////

// Internal options for early initialization of memory tracing systems. Exposed
// here due to visibility in platform implementations.
enum class EMemoryTraceInit : uint8
{
	Disabled		= 0,
	AllocEvents		= 1 << 0,
	Callstacks		= 1 << 1,
	Tags			= 1 << 2,
	Probing			= 1 << 3,
	Full			= AllocEvents|Callstacks|Tags|Probing,
	Light			= AllocEvents|Tags,
};

ENUM_CLASS_FLAGS(EMemoryTraceInit);

////////////////////////////////////////////////////////////////////////////////
#if UE_MEMORY_TRACE_ENABLED

#define UE_MEMORY_TRACE(x) x

CORE_API UE_TRACE_CHANNEL_EXTERN(MemAllocChannel);

////////////////////////////////////////////////////////////////////////////////
class FMalloc* MemoryTrace_Create(class FMalloc* InMalloc);
void MemoryTrace_Initialize();

/**
 * Returns the untracked allocator used to initialize memory tracing. Only internally used.
 * @return A valid allocator if exists, nullptr otherwise.
 */
class FMalloc* MemoryTrace_GetAllocator();

/**
 * Register a new heap specification (name). Use the returned value when marking heaps.
 * @param ParentId Heap id of parent heap.
 * @param Name Descriptive name of the heap.
 * @param Flags Properties of this heap. See \ref EMemoryTraceHeapFlags
 * @return Heap id to use when allocating memory
 */
CORE_API HeapId MemoryTrace_HeapSpec(HeapId ParentId, const TCHAR* Name, EMemoryTraceHeapFlags Flags = EMemoryTraceHeapFlags::None);

/**
 * Register a new root heap specification (name). Use the returned value as parent to other heaps.
 * @param Name Descriptive name of the root heap.
 * @param Flags Properties of the this root heap. See \ref EMemoryTraceHeapFlags
 * @return Heap id to use when allocating memory
 */
CORE_API HeapId MemoryTrace_RootHeapSpec(const TCHAR* Name, EMemoryTraceHeapFlags Flags = EMemoryTraceHeapFlags::None);

/**
 * Mark a traced allocation as being a heap.
 * @param Address Address of the allocation
 * @param Heap Heap id, see /ref MemoryTrace_HeapSpec. If no specific heap spec has been created the correct root heap needs to be given.
 * @param Flags Additional properties of the heap allocation. Note that \ref EMemoryTraceHeapAllocationFlags::Heap is implicit.
 * @param ExternalCallstackId CallstackId to use, if 0 will use current callstack id.
 */
CORE_API void MemoryTrace_MarkAllocAsHeap(uint64 Address, HeapId Heap, EMemoryTraceHeapAllocationFlags Flags = EMemoryTraceHeapAllocationFlags::None, uint32 ExternalCallstackId = 0);

/**
 * Unmark an allocation as a heap. When an allocation that has previously been used as a heap is reused as a regular
 * allocation.
 * @param Address Address of the allocation
 * @param Heap Heap id
 * @param ExternalCallstackId CallstackId to use, if 0 will use current callstack id.
 */
CORE_API void MemoryTrace_UnmarkAllocAsHeap(uint64 Address, HeapId Heap, uint32 ExternalCallstackId = 0);

/**
 * Trace an allocation event.
 * @param Address Address of allocation
 * @param Size Size of allocation
 * @param Alignment Alignment of the allocation
 * @param RootHeap Which root heap this belongs to (system memory, video memory etc)
 * @param ExternalCallstackId CallstackId to use, if 0 will use current callstack id.
 */
CORE_API void MemoryTrace_Alloc(uint64 Address, uint64 Size, uint32 Alignment, HeapId RootHeap = EMemoryTraceRootHeap::SystemMemory, uint32 ExternalCallstackId = 0);

/**
 * Trace a free event.
 * @param Address Address of the allocation being freed
 * @param RootHeap Which root heap this belongs to (system memory, video memory etc)
 * @param ExternalCallstackId CallstackId to use, if 0 will use current callstack id.
 */
CORE_API void MemoryTrace_Free(uint64 Address, HeapId RootHeap = EMemoryTraceRootHeap::SystemMemory, uint32 ExternalCallstackId = 0);

/**
 * Trace a free related to a reallocation event.
 * @param Address Address of the allocation being freed
 * @param RootHeap Which root heap this belongs to (system memory, video memory etc)
 * @param ExternalCallstackId CallstackId to use, if 0 will use current callstack id.
 */
CORE_API void MemoryTrace_ReallocFree(uint64 Address, HeapId RootHeap = EMemoryTraceRootHeap::SystemMemory, uint32 ExternalCallstackId = 0);

/** Trace an allocation related to a reallocation event.
 * @param Address Address of allocation
 * @param NewSize Size of allocation
 * @param Alignment Alignment of the allocation
 * @param RootHeap Which root heap this belongs to (system memory, video memory etc)
 * @param ExternalCallstackId CallstackId to use, if 0 will use current callstack id.
 */
CORE_API void MemoryTrace_ReallocAlloc(uint64 Address, uint64 NewSize, uint32 Alignment, HeapId RootHeap = EMemoryTraceRootHeap::SystemMemory, uint32 ExternalCallstackId = 0);

/**
 * Trace an update alloc event. It updates context (mem tag and metadata) for an allocation.
 * @param Address Address of the allocation being updated
 * @param RootHeap Which root heap this belongs to (system memory, video memory etc)
 * @param ExternalCallstackId CallstackId to use, if 0 will use current callstack id.
 */
CORE_API void MemoryTrace_UpdateAlloc(uint64 Address, HeapId RootHeap = EMemoryTraceRootHeap::SystemMemory, uint32 ExternalCallstackId = 0);

/** Trace a swap operation. Only available for system memory root heap (EMemoryTraceRootHeap::SystemMemory).
 * @param PageAddress Page address for operation, in case of PageIn can be address of the page fault (not aligned to page boundary).
 * @param SwapOperation Which swap operation is happening to the address.
 * @param CompressedSize Compressed size of the page for page out operation.
 * @param CallstackId CallstackId to use, if 0 to ignore (will not use current callstack id).
 */
CORE_API void MemoryTrace_SwapOp(uint64 PageAddress, EMemoryTraceSwapOperation SwapOperation, uint32 CompressedSize = 0, uint32 CallstackId = 0);

////////////////////////////////////////////////////////////////////////////////
#else // UE_MEMORY_TRACE_ENABLED

#define UE_MEMORY_TRACE(x)
inline HeapId MemoryTrace_RootHeapSpec(const TCHAR* Name, EMemoryTraceHeapFlags Flags = EMemoryTraceHeapFlags::None) { return ~0; };
inline HeapId MemoryTrace_HeapSpec(HeapId ParentId, const TCHAR* Name, EMemoryTraceHeapFlags Flags = EMemoryTraceHeapFlags::None) { return ~0; }
inline void MemoryTrace_MarkAllocAsHeap(uint64 Address, HeapId Heap) {}
inline void MemoryTrace_UnmarkAllocAsHeap(uint64 Address, HeapId Heap) {}
inline void MemoryTrace_Alloc(uint64 Address, uint64 Size, uint32 Alignment, HeapId RootHeap = EMemoryTraceRootHeap::SystemMemory, uint32 ExternalCallstackId = 0) {}
inline void MemoryTrace_Free(uint64 Address, HeapId RootHeap = EMemoryTraceRootHeap::SystemMemory, uint32 ExternalCallstackId = 0) {}
inline void MemoryTrace_ReallocFree(uint64 Address, HeapId RootHeap = EMemoryTraceRootHeap::SystemMemory, uint32 ExternalCallstackId = 0) {}
inline void MemoryTrace_ReallocAlloc(uint64 Address, uint64 NewSize, uint32 Alignment, HeapId RootHeap = EMemoryTraceRootHeap::SystemMemory, uint32 ExternalCallstackId = 0) {}
inline void MemoryTrace_UpdateAlloc(uint64 Address, HeapId RootHeap = EMemoryTraceRootHeap::SystemMemory, uint32 ExternalCallstackId = 0) {}
inline void MemoryTrace_SwapOp(uint64 PageAddress, EMemoryTraceSwapOperation SwapOperation, uint32 CompressedSize = 0, uint32 CallstackId = 0) {}
inline class FMalloc* MemoryTrace_GetAllocator() { return nullptr; }

#endif // UE_MEMORY_TRACE_ENABLED
