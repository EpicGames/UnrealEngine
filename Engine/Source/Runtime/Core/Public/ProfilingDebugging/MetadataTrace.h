// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ProfilingDebugging/StringsTrace.h"
#include "Trace/Config.h"
#include "Trace/Trace.h"

# if !defined(UE_TRACE_METADATA_ENABLED)
	#if !IS_PROGRAM
		#define UE_TRACE_METADATA_ENABLED UE_TRACE_ENABLED
	#else
		#define UE_TRACE_METADATA_ENABLED 0
	#endif
#endif //defined(UE_TRACE_METADATA_ENABLED

#if UE_TRACE_METADATA_ENABLED

// Only needed when enabled
#include "Trace/Trace.inl"

namespace UE { namespace Trace { class FChannel; } }

/**
 * Channel that enables the metadata tracing
 */
CORE_API UE_TRACE_CHANNEL_EXTERN(MetadataChannel);

CORE_API UE_TRACE_EVENT_BEGIN_EXTERN(MetadataStack, ClearScope, NoSync)
UE_TRACE_EVENT_END()

CORE_API UE_TRACE_EVENT_BEGIN_EXTERN(MetadataStack, SaveStack)
	UE_TRACE_EVENT_FIELD(uint32, Id)
UE_TRACE_EVENT_END()

CORE_API UE_TRACE_EVENT_BEGIN_EXTERN(MetadataStack, RestoreStack)
	UE_TRACE_EVENT_FIELD(uint32, Id)
UE_TRACE_EVENT_END()

class FMetadataTrace
{
public:
	CORE_API static uint32 SaveStack();
};

// For now just forward to the event definition macros.
#define UE_TRACE_METADATA_EVENT_BEGIN_EXTERN(ScopeName, ...) UE_TRACE_EVENT_BEGIN_EXTERN(Metadata, ScopeName, ##__VA_ARGS__)
#define UE_TRACE_METADATA_EVENT_DEFINE(ScopeName) UE_TRACE_EVENT_DEFINE(Metadata, ScopeName)
#define UE_TRACE_METADATA_EVENT_BEGIN(ScopeName, ...) UE_TRACE_EVENT_BEGIN(Metadata, ScopeName, ##__VA_ARGS__)
#define UE_TRACE_METADATA_EVENT_FIELD(Type, Name) UE_TRACE_EVENT_FIELD(Type, Name)
#define UE_TRACE_METADATA_EVENT_REFERENCE_FIELD(RefLogger, RefEventName, Name) UE_TRACE_EVENT_REFERENCE_FIELD(RefLogger, RefEventName, Name)
#define UE_TRACE_METADATA_EVENT_END() UE_TRACE_EVENT_END()

/**
 * A scope to push/pop data on the metadata stack.
 */
#define UE_TRACE_METADATA_SCOPE(ScopeName, ChannelExpr) \
	UE_TRACE_LOG_SCOPED(Metadata, ScopeName, MetadataChannel | ChannelExpr)

/**
 * Temporarily clears the current metadata stack and starts a new one. When this scope is exited the previous
 * stack is restored. This is useful for example in situations where an asset causes allocation in an unrelated
 * system, like shared buffers, but that allocation should not be ascribed to a specific asset.
 */
#define UE_TRACE_METADATA_CLEAR_SCOPE() \
	UE_TRACE_LOG_SCOPED(MetadataStack, ClearScope, MetadataChannel)

/**
 * Saves the current metadata stack, and returns an identifier. The stack can be restored on another thread using
 * the identifier. This can be used when the context needs to be applied on a task.
 */
#define UE_TRACE_METADATA_SAVE_STACK() \
	FMetadataTrace::SaveStack();

/**
 * A scope that restores a metadata stack previously stored with UE_TRACE_METADATA_SAVE_STACK during it's lifetime.
 */
class FMetadataRestoreScope
{
public:
	CORE_API FMetadataRestoreScope(uint32 InSavedMetadataIdentifier);
	CORE_API ~FMetadataRestoreScope();
private:
	void ActivateScope(uint32 InStackId);
	UE::Trace::Private::FScopedLogScope Inner;
};

/**
 * Temporarily applies a previously saved metadata stack. See UE_TRACE_METADATA_SAVE_STACK.
 * When this scope is exited, the previous metadata stack is restored.
 */
#define UE_TRACE_METADATA_RESTORE_SCOPE(SavedMetadataIdentifier) \
	FMetadataRestoreScope UE_JOIN(__MetadataRestoreScope, __LINE__)(SavedMetadataIdentifier);

#else // UE_TRACE_METADATA_ENABLED

#define UE_TRACE_METADATA_EVENT_BEGIN_EXTERN(ScopeName, ...)
#define UE_TRACE_METADATA_EVENT_DEFINE(ScopeName)
#define UE_TRACE_METADATA_EVENT_BEGIN(ScopeName, ...)
#define UE_TRACE_METADATA_EVENT_FIELD(Type, Name)
#define UE_TRACE_METADATA_EVENT_END()
#define UE_TRACE_METADATA_SCOPE(ScopeName, ChannelExpr)

#define UE_TRACE_METADATA_CLEAR_SCOPE()
#define UE_TRACE_METADATA_SAVE_STACK() 0
#define UE_TRACE_METADATA_RESTORE_SCOPE(SavedMetadataIdentifier)

#endif // UE_TRACE_METADATA_ENABLED
