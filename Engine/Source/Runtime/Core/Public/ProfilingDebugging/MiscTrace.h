// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "CoreTypes.h"
#include "HAL/PreprocessorHelpers.h"
#include "Misc/Build.h"
#include "ProfilingDebugging/CallstackTrace.h"
#include "ProfilingDebugging/FormatArgsTrace.h"
#include "Trace/Config.h"

#if !defined(MISCTRACE_ENABLED)
#if UE_TRACE_ENABLED && !UE_BUILD_SHIPPING
#define MISCTRACE_ENABLED 1
#else
#define MISCTRACE_ENABLED 0
#endif
#endif

#ifndef FRAME_TRACE_ENABLED
#define FRAME_TRACE_ENABLED MISCTRACE_ENABLED
#endif

UE_TRACE_CHANNEL_EXTERN(BookmarkChannel, CORE_API)

enum ETraceFrameType
{
	TraceFrameType_Game,
	TraceFrameType_Rendering,

	TraceFrameType_Count
};

struct FTraceUtils
{
	static void Encode7bit(uint64 Value, uint8*& BufferPtr)
	{
		// Writes 1 to 10 bytes for uint64 and 1 to 5 bytes for uint32.
		do
		{
			uint8 HasMoreBytes = (uint8)((Value > uint64(0x7F)) << 7);
			*(BufferPtr++) = (uint8)(Value & 0x7F) | HasMoreBytes;
			Value >>= 7;
		} while (Value > 0);
	}

	static void EncodeZigZag(int64 Value, uint8*& BufferPtr)
	{
		Encode7bit((Value << 1) ^ (Value >> 63), BufferPtr);
	}
};

class FName;

struct FMiscTrace
{
	CORE_API static void OutputBookmarkSpec(const void* BookmarkPoint, const ANSICHAR* File, int32 Line, const TCHAR* Format);

	template <typename... Types>
	static void OutputBookmark(uint32 CallstackId, const void* BookmarkPoint, Types... FormatArgs)
	{
		uint8 FormatArgsBuffer[4096];
		uint16 FormatArgsSize = FFormatArgsTrace::EncodeArguments(FormatArgsBuffer, FormatArgs...);
		if (FormatArgsSize)
		{
			OutputBookmarkInternal(BookmarkPoint, CallstackId, FormatArgsSize, FormatArgsBuffer);
		}
	}

	template <typename... Types>
	static void OutputBookmarkCycles(uint32 CallstackId, uint64 Cycles, const void* BookmarkPoint, Types... FormatArgs)
	{
		uint8 FormatArgsBuffer[4096];
		uint16 FormatArgsSize = FFormatArgsTrace::EncodeArguments(FormatArgsBuffer, FormatArgs...);
		if (FormatArgsSize)
		{
			OutputBookmarkInternalCycles(Cycles, BookmarkPoint, CallstackId, FormatArgsSize, FormatArgsBuffer);
		}
	}

	CORE_API static void OutputBeginRegion(const TCHAR* RegionName, const TCHAR* Category = nullptr);
	[[nodiscard]] CORE_API static uint64 OutputBeginRegionWithId(const TCHAR* RegionName, const TCHAR* Category = nullptr);
	CORE_API static void OutputEndRegion(const TCHAR* RegionName);
	CORE_API static void OutputEndRegionWithId(uint64 RegionId);

	CORE_API static void OutputBeginFrame(ETraceFrameType FrameType);
	CORE_API static void OutputEndFrame(ETraceFrameType FrameType);

	CORE_API static void OutputScreenshot(const TCHAR* Name, uint64 Cycle, uint32 Width, uint32 Height, TArray64<uint8> Data);
	CORE_API static bool ShouldTraceScreenshot();
	CORE_API static bool ShouldTraceBookmark();
	CORE_API static bool ShouldTraceRegion();

private:
	CORE_API static void OutputBookmarkInternal(const void* BookmarkPoint, uint32 CallstackId, uint16 EncodedFormatArgsSize, uint8* EncodedFormatArgs);
	CORE_API static void OutputBookmarkInternalCycles(uint64 Cycles, const void* BookmarkPoint, uint32 CallstackId, uint16 EncodedFormatArgsSize, uint8* EncodedFormatArgs);
};

#if MISCTRACE_ENABLED

#define TRACE_BOOKMARK(Format, ...) \
if (UE_TRACE_CHANNELEXPR_IS_ENABLED(BookmarkChannel)) \
{ \
	static bool UE_JOIN(__BookmarkPoint, __LINE__); \
	if (!UE_JOIN(__BookmarkPoint, __LINE__)) \
	{ \
		static_assert(std::is_const_v<std::remove_reference_t<decltype(Format)>>, "Formatting string must be a const TCHAR array."); \
		static_assert(TIsArrayOrRefOfTypeByPredicate<decltype(Format), TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array."); \
		UE_VALIDATE_FORMAT_STRING(Format, ##__VA_ARGS__); \
		FMiscTrace::OutputBookmarkSpec(&UE_JOIN(__BookmarkPoint, __LINE__), __FILE__, __LINE__, Format); \
		UE_JOIN(__BookmarkPoint, __LINE__) = true; \
	} \
	FMiscTrace::OutputBookmark(CallstackTrace_GetCurrentId(), &UE_JOIN(__BookmarkPoint, __LINE__), ##__VA_ARGS__); \
}

#define TRACE_BOOKMARK_CYCLES(Cycles, Format, ...) \
if (UE_TRACE_CHANNELEXPR_IS_ENABLED(BookmarkChannel)) \
{ \
	static bool UE_JOIN(__BookmarkPoint, __LINE__); \
	if (!UE_JOIN(__BookmarkPoint, __LINE__)) \
	{ \
		static_assert(std::is_const_v<std::remove_reference_t<decltype(Format)>>, "Formatting string must be a const TCHAR array."); \
		static_assert(TIsArrayOrRefOfTypeByPredicate<decltype(Format), TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array."); \
		UE_VALIDATE_FORMAT_STRING(Format, ##__VA_ARGS__); \
		FMiscTrace::OutputBookmarkSpec(&UE_JOIN(__BookmarkPoint, __LINE__), __FILE__, __LINE__, Format); \
		UE_JOIN(__BookmarkPoint, __LINE__) = true; \
	} \
	FMiscTrace::OutputBookmarkCycles(CallstackTrace_GetCurrentId(), Cycles, &UE_JOIN(__BookmarkPoint, __LINE__), ##__VA_ARGS__); \
}

/**
 * Start a Timing Region, identified by its name.
 * Prefer the _WITH_ID variant of this macro since overlapping Regions with the same name cannot be uniquely identified
 * Example Usage:
 * @code
 * TRACE_BEGIN_REGION(TEXT("MyRegion"))
 * [...]
 * TRACE_END_REGION(TEXT("MyRegion"))
 * @endcode 
 * 
 */
#define TRACE_BEGIN_REGION(RegionName, ...) \
FMiscTrace::OutputBeginRegion(RegionName __VA_OPT__(,) __VA_ARGS__);

/**
 * Start a Timing Region with a name and optionally a category. Returns an Id that may be used with TRACE_END_REGION_WITH_ID to end tracking.
 * Both the name and the category should be null-terminated TCHAR* strings, and may be dynamic strings or string constants.
 * Example Usage:
 * @code
 * uint64 OpenRegionId = TRACE_BEGIN_REGION_WITH_ID(TEXT("MyRegion"), TEXT("MyCategory"))
 * [...]
 * TRACE_END_REGION_WITH_ID(OpenRegionId)
 * @endcode 
 * 
 */
#define TRACE_BEGIN_REGION_WITH_ID(RegionName, ...) \
FMiscTrace::OutputBeginRegionWithId(RegionName __VA_OPT__(,) __VA_ARGS__);

#define TRACE_END_REGION(RegionName) \
	FMiscTrace::OutputEndRegion(RegionName);

#define TRACE_END_REGION_WITH_ID(RegionId) \
	FMiscTrace::OutputEndRegionWithId(RegionId);

#define TRACE_SCREENSHOT(Name, Cycle, Width, Height, Data) \
	FMiscTrace::OutputScreenshot(Name, Cycle, Width, Height, Data);

#define SHOULD_TRACE_SCREENSHOT() \
	FMiscTrace::ShouldTraceScreenshot()

#define SHOULD_TRACE_BOOKMARK() \
	FMiscTrace::ShouldTraceBookmark()

#define SHOULD_TRACE_REGION() \
	FMiscTrace::ShouldTraceRegion()

#else

#define TRACE_BOOKMARK(...)
#define TRACE_BOOKMARK_CYCLES(...)
#define TRACE_BEGIN_REGION(...)
#define TRACE_BEGIN_REGION_WITH_ID(...) 0;
#define TRACE_END_REGION(...)
#define TRACE_END_REGION_WITH_ID(...)
#define TRACE_SCREENSHOT(...)
#define SHOULD_TRACE_SCREENSHOT(...) false
#define SHOULD_TRACE_BOOKMARK(...) false
#define SHOULD_TRACE_REGION(...) false

#endif // MISCTRACE_ENABLED

#if FRAME_TRACE_ENABLED 

#define TRACE_BEGIN_FRAME(FrameType) \
FMiscTrace::OutputBeginFrame(FrameType);

#define TRACE_END_FRAME(FrameType) \
FMiscTrace::OutputEndFrame(FrameType);

#else

#define TRACE_BEGIN_FRAME(...)
#define TRACE_END_FRAME(...)

#endif // FRAME_TRACE_ENABLED
