// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"

class FString;

#if !defined(PLATFORM_SUPPORTS_PLATFORM_EVENTS)
#	define PLATFORM_SUPPORTS_PLATFORM_EVENTS 0
#endif

/////////////////////////////////////////////////////////////////////

UE_TRACE_CHANNEL_CUSTOM_EXTERN(ContextSwitchChannel, FCSChannel)
UE_TRACE_CHANNEL_CUSTOM_EXTERN(StackSamplingChannel, FSSChannel)

/////////////////////////////////////////////////////////////////////

class FPlatformEventsTrace
{
public:
	enum class EEventType
	{
		None = 0x00,
		ContextSwitch = 0x01,
		StackSampling = 0x02,
	};

	CORE_API static EEventType GetEvent(const FString& Name);

#if PLATFORM_SUPPORTS_PLATFORM_EVENTS

	CORE_API static void Init(uint32 SamplingIntervalUsec);
	CORE_API static void PostInit();
	CORE_API static bool CanEnable(EEventType Event, const TCHAR** OutReason);
	CORE_API static void Enable(EEventType Event);
	CORE_API static void Disable(EEventType Event);
	CORE_API static void Stop();
	UE_DEPRECATED(5.7, "Use channel callback instead.")
	CORE_API static void OnTraceChannelUpdated(const FString& ChannelName, bool bIsEnabled);

	CORE_API static void OutputSettings(uint32 SamplingIntervalUsec);
	CORE_API static void OutputContextSwitch(uint64 StartTime, uint64 EndTime, uint32 ThreadId, uint8 CoreNumber);
	CORE_API static void OutputStackSample(uint64 Time, uint32 ThreadId, const uint64* Addresses, uint32 AddressCount);
	CORE_API static void OutputThreadName(uint32 ThreadId, uint32 ProcessId, const TCHAR* Name, uint32 NameLen);

#else // PLATFORM_SUPPORTS_PLATFORM_EVENTS

	static void Init(uint32 SamplingIntervalUsec) {}
	static void PostInit() {}
	static void Enable(EEventType Event) {}
	static void Disable(EEventType Event) {}
	static void Stop() {}
	static void OnTraceChannelUpdated(const FString& ChannelName, bool bIsEnabled) {}

	static void OutputContextSwitch(uint64 StartTime, uint64 EndTime, uint32 ThreadId, uint8 CoreNumber) {}
	static void OutputStackSample(uint64 Time, uint32 ThreadId, const uint64* Addresses, uint32 AddressCount) {}
	static void OutputThreadName(uint32 ThreadId, uint32 ProcessId, const TCHAR* Name, uint32 NameLen) {}

#endif // PLATFORM_SUPPORTS_PLATFORM_EVENTS
};

ENUM_CLASS_FLAGS(FPlatformEventsTrace::EEventType)


/////////////////////////////////////////////////////////////////////
