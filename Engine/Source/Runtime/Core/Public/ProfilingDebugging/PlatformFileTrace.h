// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Build.h"
#include "Trace/Config.h"

#if !UE_BUILD_SHIPPING
#define PLATFORMFILETRACE_ENABLED 1
#else
#define PLATFORMFILETRACE_ENABLED 0
#endif

#if PLATFORMFILETRACE_ENABLED

struct FPlatformFileTrace
{
	static CORE_API void BeginOpen(const TCHAR* Path);
	static CORE_API void EndOpen(uint64 FileHandle);
	static CORE_API void FailOpen(const TCHAR* Path);
	static CORE_API void BeginReOpen(uint64 OldFileHandle);
	static CORE_API void EndReOpen(uint64 NewFileHandle);
	static CORE_API void BeginClose(uint64 FileHandle);
	static CORE_API void EndClose(uint64 FileHandle);
	static CORE_API void FailClose(uint64 FileHandle);
	static CORE_API void BeginRead(uint64 ReadHandle, uint64 FileHandle, uint64 Offset, uint64 Size);
	static CORE_API void EndRead(uint64 ReadHandle, uint64 SizeRead);
	static CORE_API void BeginWrite(uint64 WriteHandle, uint64 FileHandle, uint64 Offset, uint64 Size);
	static CORE_API void EndWrite(uint64 WriteHandle, uint64 SizeWritten);

	static CORE_API uint32 GetOpenFileHandleCount();
};

#define TRACE_PLATFORMFILE_BEGIN_OPEN(Path) \
	FPlatformFileTrace::BeginOpen(Path);

#define TRACE_PLATFORMFILE_END_OPEN(FileHandle) \
	FPlatformFileTrace::EndOpen(uint64(FileHandle));

#define TRACE_PLATFORMFILE_FAIL_OPEN(Path) \
	FPlatformFileTrace::FailOpen(Path);

#define TRACE_PLATFORMFILE_BEGIN_REOPEN(OldFileHandle) \
	FPlatformFileTrace::BeginReOpen(uint64(OldFileHandle));

#define TRACE_PLATFORMFILE_END_REOPEN(NewFileHandle) \
	FPlatformFileTrace::EndReOpen(uint64(NewFileHandle));

#define TRACE_PLATFORMFILE_BEGIN_CLOSE(FileHandle) \
	FPlatformFileTrace::BeginClose(uint64(FileHandle));

#define TRACE_PLATFORMFILE_END_CLOSE(FileHandle) \
	FPlatformFileTrace::EndClose(uint64(FileHandle));

#define TRACE_PLATFORMFILE_FAIL_CLOSE(FileHandle) \
	FPlatformFileTrace::FailClose(uint64(FileHandle));

#define TRACE_PLATFORMFILE_BEGIN_READ(ReadHandle, FileHandle, Offset, Size) \
	FPlatformFileTrace::BeginRead(uint64(ReadHandle), uint64(FileHandle), Offset, Size);

#define TRACE_PLATFORMFILE_END_READ(ReadHandle, SizeRead) \
	FPlatformFileTrace::EndRead(uint64(ReadHandle), SizeRead);

#define TRACE_PLATFORMFILE_BEGIN_WRITE(WriteHandle, FileHandle, Offset, Size) \
	FPlatformFileTrace::BeginWrite(uint64(WriteHandle), uint64(FileHandle), Offset, Size);

#define TRACE_PLATFORMFILE_END_WRITE(WriteHandle, SizeWritten) \
	FPlatformFileTrace::EndWrite(uint64(WriteHandle), SizeWritten);

#else

#define TRACE_PLATFORMFILE_BEGIN_OPEN(Path)
#define TRACE_PLATFORMFILE_END_OPEN(FileHandle)
#define TRACE_PLATFORMFILE_BEGIN_REOPEN(OldFileHandle)
#define TRACE_PLATFORMFILE_END_REOPEN(NewFileHandle)
#define TRACE_PLATFORMFILE_FAIL_OPEN(Path)
#define TRACE_PLATFORMFILE_BEGIN_CLOSE(FileHandle)
#define TRACE_PLATFORMFILE_END_CLOSE(FileHandle)
#define TRACE_PLATFORMFILE_FAIL_CLOSE(FileHandle)
#define TRACE_PLATFORMFILE_BEGIN_READ(ReadHandle, FileHandle, Offset, Size)
#define TRACE_PLATFORMFILE_END_READ(ReadHandle, SizeRead)
#define TRACE_PLATFORMFILE_BEGIN_WRITE(WriteHandle, FileHandle, Offset, Size)
#define TRACE_PLATFORMFILE_END_WRITE(WriteHandle, SizeWritten)

#endif
