// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Here are a number of profiling helper functions so we do not have to duplicate a lot of the glue
 * code everywhere.  And we can have consistent naming for all our files.
 *
 */

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"

enum EStreamingStatus
{
	LEVEL_Unloaded,
	LEVEL_UnloadedButStillAround,
	LEVEL_Loading,
	LEVEL_Loaded,
	LEVEL_MakingVisible,
	LEVEL_Visible,
	LEVEL_Preloading,
	LEVEL_FailedToLoad,
	LEVEL_MakingInvisible,
	LEVEL_StreamingStatusCount
};


/**
 * This will get the version that should be used with the Automated Performance Testing
 * If one is passed in we use that otherwise we use FApp::GetEngineVersion.  This allows
 * us to have build machine built .exe and test them. 
 *
 * NOTE: had to use AutomatedBenchmarking as the parsing code is flawed and doesn't match
 *       on whole words.  so automatedperftestingChangelist was failing :-( 
 **/
CORE_API int32 GetChangeListNumberForPerfTesting();

/**
 * This will return
 *   the name specified with -BuildName=SomeName (needs to be a valid c++ name a-z A-Z _ 0-9 so we don't run into issues when we pass it around) or if not specified
 *   "CL%d" where %d is coming from GetChangeListNumberForPerfTesting()
 **/
CORE_API FString GetBuildNameForPerfTesting();

/**
 * This makes it so UnrealConsole will open up the memory profiler for us
 *
 * @param NotifyType has the <namespace>:<type> (e.g. UE_PROFILER!UE3STATS:)
 * @param FullFileName the File name to copy from the console
 **/
CORE_API void SendDataToPCViaUnrealConsole( const FString& NotifyType, const FString& FullFileName );


/** 
 * This will generate the profiling file name that will work with limited filename sizes on consoles.
 * We want a uniform naming convention so we will all just call this function.
 *
 * @param ProfilingType this is the type of profiling file this is
 * 
 **/
CORE_API FString CreateProfileFilename( const FString& InFileExtension, bool bIncludeDateForDirectoryName );

/**
* This will generate the profiling file name that will work with limited filename sizes on consoles.
* We want a uniform naming convention so we will all just call this function.
*
*
**/
CORE_API FString CreateProfileFilename(const FString& InFilename, const FString& InFileExtension, bool bIncludeDateForDirectoryName);

/** 
 * This will create the directories and the file name all in one function
 **/
CORE_API FString CreateProfileDirectoryAndFilename( const FString& InSubDirectoryName, const FString& InFileExtension );

#if WITH_ENGINE
/** Delegate type for getting current map name */
DECLARE_DELEGATE_RetVal(const FString, FGetMapNameDelegate);

/** Delegate used by CreateProfileFilename() and CreateProfileDirectoryAndFilename() to get current map name */
extern CORE_API FGetMapNameDelegate GGetMapNameDelegate;
#endif

