// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"

class FName;
class FOutputDevice;

/** Indicate what types of resources should be included for calculating used memory */
namespace EResourceSizeMode
{
	enum Type
	{
		/** Only include memory used by non-UObject resources that are directly owned by this UObject. This is used to show memory actually used at runtime */
		Exclusive,
		/** Include exclusive resources and UObject serialized memory for this and all child UObjects, but not memory for external referenced assets or editor only members. This is used in the editor to estimate maximum required memory */
		EstimatedTotal,
	};
};

/**
 * Struct used to count up the amount of memory used by a resource.
 * This is typically used for assets via UObject::GetResourceSizeEx(...).
 */
struct FResourceSizeEx
{
public:

	/**
	 * Default constructor. 
	 */
	CORE_API explicit FResourceSizeEx();

	/**
	 * Construct using a given mode. 
	 */
	CORE_API explicit FResourceSizeEx(const EResourceSizeMode::Type InResourceSizeMode);

	/**
	 * Construct from known sizes. 
	 */
	CORE_API FResourceSizeEx(const EResourceSizeMode::Type InResourceSizeMode, const SIZE_T InDedicatedSystemMemoryBytes, const SIZE_T InDedicatedVideoMemoryBytes);

	/**
	 * Construct from legacy unknown size.
	 * Deliberately explicit to avoid accidental use.
	 */
	CORE_API FResourceSizeEx(const EResourceSizeMode::Type InResourceSizeMode, const SIZE_T InUnknownMemoryBytes);

	CORE_API void LogSummary(FOutputDevice& Ar) const;

	/**
	 * Get the type of resource size held in this struct.
	 */
	CORE_API EResourceSizeMode::Type GetResourceSizeMode() const;

	CORE_API FResourceSizeEx& AddDedicatedSystemMemoryBytes(const FName& Tag, const SIZE_T InMemoryBytes);

	/**
	 * Add the given number of bytes to the dedicated system memory count.
	 * @see DedicatedSystemMemoryBytes for a description of that memory type.
	 */
	CORE_API FResourceSizeEx& AddDedicatedSystemMemoryBytes(const SIZE_T InMemoryBytes);

	/**
	 * Get the number of bytes allocated from dedicated system memory.
	 * @see DedicatedSystemMemoryBytes for a description of that memory type.
	 */
	CORE_API SIZE_T GetDedicatedSystemMemoryBytes() const;

	CORE_API FResourceSizeEx& AddDedicatedVideoMemoryBytes(const FName& Tag, const SIZE_T InMemoryBytes);

	/**
	 * Add the given number of bytes to the dedicated video memory count.
	 * @see DedicatedVideoMemoryBytes for a description of that memory type.
	 */
	CORE_API FResourceSizeEx& AddDedicatedVideoMemoryBytes(const SIZE_T InMemoryBytes);

	/**
	 * Get the number of bytes allocated from dedicated video memory.
	 * @see DedicatedVideoMemoryBytes for a description of that memory type.
	 */
	CORE_API SIZE_T GetDedicatedVideoMemoryBytes() const;

	CORE_API FResourceSizeEx& AddUnknownMemoryBytes(const FName& Tag, const SIZE_T InMemoryBytes);

	/**
	 * Add the given number of bytes to the unknown memory count.
	 * @see UnknownMemoryBytes for a description of that memory type.
	 */
	CORE_API FResourceSizeEx& AddUnknownMemoryBytes(const SIZE_T InMemoryBytes);

	/**
	 * Get the number of bytes allocated from unknown memory.
	 * @see UnknownMemoryBytes for a description of that memory type.
	 */
	CORE_API SIZE_T GetUnknownMemoryBytes() const;

	/**
	 * Get the total number of bytes allocated from any memory.
	 */
	CORE_API SIZE_T GetTotalMemoryBytes() const;

	/**
	 * Add another FResourceSizeEx to this one.
	 */
	CORE_API FResourceSizeEx& operator+=(const FResourceSizeEx& InRHS);

private:
	/**
	 * Type of resource size held in this struct.
	 */
	EResourceSizeMode::Type ResourceSizeMode;

	/**
	 * The number of bytes of memory that this resource is using for CPU resources that have been allocated from dedicated system memory.
	 * On platforms with unified memory, this typically refers to the things allocated in the preferred memory for CPU use.
	 */
	TMap<FName, SIZE_T> DedicatedSystemMemoryBytesMap;

	/**
	 * The number of bytes of memory that this resource is using for GPU resources that have been allocated from dedicated video memory.
	 * On platforms with unified memory, this typically refers to the things allocated in the preferred memory for GPU use.
	 */
	TMap<FName, SIZE_T> DedicatedVideoMemoryBytesMap;

	/**
	 * The number of bytes of memory that this resource is using from an unspecified section of memory.
	 * This exists so that the legacy GetResourceSize(...) functions can still report back memory usage until they're updated to use FResourceSizeEx, and should not be used in new memory tracking code.
	 */
	TMap<FName, SIZE_T> UnknownMemoryBytesMap;
};

/**
* Add two FResourceSizeEx instances together and return a copy.
*/
CORE_API FResourceSizeEx operator+(FResourceSizeEx InLHS, const FResourceSizeEx& InRHS);
