// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScopedDebugInfo.h: Scoped debug info definitions.
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"

/** Debug info that is stored on the stack and handled by stack unwinding. */
class FScopedDebugInfo
{
public:

	enum { MaxDescriptionLength = 128 };

	/** The number of calls on the stack the debug info replaces. */
	const int32 NumReplacedOuterCalls;

	/** The next debug info on the stack, or NULL if this is the last debug info on the stack. */
	FScopedDebugInfo* const NextOuterInfo;

	/** Initialization constructor. */
	CORE_API FScopedDebugInfo(int32 InNumReplacedOuterCalls);

	/** Destructor. */
	CORE_API virtual ~FScopedDebugInfo();

	/** @return The "function name" to display on the call stack for this debug info. */
	virtual FString GetFunctionName() const = 0;

	/** @return The filename to display on the call stack for this debug info. */
	virtual FString GetFilename() const = 0;

	/** @return The line number to display on the call stack for this debug info. */
	virtual int32 GetLineNumber() const = 0;

	/** Accesses the list of debug infos on the stack in this thread. */
	static CORE_API FScopedDebugInfo* GetDebugInfoStack();
};

