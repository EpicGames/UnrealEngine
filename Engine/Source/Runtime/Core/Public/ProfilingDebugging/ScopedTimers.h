// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Containers/StringFwd.h"
#include "CoreGlobals.h"
#include "HAL/PlatformTime.h"
#include "HAL/ThreadSafeCounter.h"
#include "Misc/OutputDeviceRedirector.h"
#include "HAL/ThreadSafeCounter64.h"

#include <atomic>

/**
 * Implements the same behavior as std::atomic<double>.fetch_add(double Delta), which is only available in c++20.
 * This function will be deprecated and callsites will replace it with std::atomic<double>.fetch_add when versions
 * earlier than c++20 are deprecated.
 * 
 * Atomically adds Delta to Value, using the given MemoryOrder for the ReadModifyWrite of the new value.
 * @return The value before the finally successful addition.
 */
double AtomicDoubleFetchAdd(std::atomic<double>& Value, double Delta,
	std::memory_order MemoryOrder = std::memory_order_seq_cst);

/**
 * Utility stopwatch class for tracking the duration of some action (tracks 
 * time in seconds and adds it to the specified variable on destruction).
 */
class FDurationTimer
{
public:
	explicit FDurationTimer(double& AccumulatorIn)
		: StartTime(FPlatformTime::Seconds())
		, Accumulator(AccumulatorIn)
	{}

	double Start()
	{
		StartTime = FPlatformTime::Seconds();
		return StartTime;
	}

	double Stop()
	{
		double StopTime = FPlatformTime::Seconds();
		Accumulator += (StopTime - StartTime);
		StartTime = StopTime;
			
		return StopTime;
	}

protected:
	/** Start time, captured in ctor. */
	double StartTime;
	/** Time variable to update. */
	double& Accumulator;
};

/**
 * Utility class for tracking the duration of a scoped action (the user 
 * doesn't have to call Start() and Stop() manually).
 */
class FScopedDurationTimer : public FDurationTimer
{
public:
	explicit FScopedDurationTimer(double& AccumulatorIn)
		: FDurationTimer(AccumulatorIn)
	{
	}

	/** Dtor, updating seconds with time delta. */
	~FScopedDurationTimer()
	{
		Stop();
	}
};

/**
 * Same as FScopedDurationTimer, except that it tracks the time value internally so you don't have to
 * pass in a double to accumulate.  Call GetTime() to get the total time since starting.
 */
class FAutoScopedDurationTimer : public FScopedDurationTimer
{
public:
	FAutoScopedDurationTimer()
		: FScopedDurationTimer(AccumulatorValue)
		, AccumulatorValue(0)
	{
	}

	double GetTime()
	{
		Stop();
		return AccumulatorValue;
	}

private:
	double AccumulatorValue;
};


/**
 * Utility stopwatch class for tracking the duration of some action (tracks time in seconds and adds it to the
 * specified variable on destruction), when the storage is std::atomic<double>.
 */
class FDurationAtomicTimer
{
public:
	explicit FDurationAtomicTimer(std::atomic<double>& AccumulatorIn,
		std::memory_order InMemoryOrder = std::memory_order_relaxed)
		: StartTime(FPlatformTime::Seconds())
		, Accumulator(AccumulatorIn)
		, MemoryOrder(InMemoryOrder)
	{}

	double Start()
	{
		StartTime = FPlatformTime::Seconds();
		return StartTime;
	}

	double Stop()
	{
		double StopTime = FPlatformTime::Seconds();
		AtomicDoubleFetchAdd(Accumulator, StopTime - StartTime, MemoryOrder);
		StartTime = StopTime;

		return StopTime;
	}

protected:
	/** Start time, captured in ctor. */
	double StartTime;
	/** Time variable to update. */
	std::atomic<double>& Accumulator;
	/** Memory order specified in constructor and passed to fetch_add. */
	std::memory_order MemoryOrder;
};

/**
 * Utility class for tracking the duration of a scoped action (the user doesn't have to call Start() and
 * Stop() manually), when the storage is std::atomic<double>.
 */
class FScopedDurationAtomicTimer : public FDurationAtomicTimer
{
public:
	explicit FScopedDurationAtomicTimer(std::atomic<double>& AccumulatorIn,
		std::memory_order InMemoryOrder = std::memory_order_relaxed)
		: FDurationAtomicTimer(AccumulatorIn, InMemoryOrder)
	{
	}

	/** Dtor, updating seconds with time delta. */
	~FScopedDurationAtomicTimer()
	{
		Stop();
	}
};

/**
 * Utility class for tracking the duration of a scoped action to an accumulator in a thread-safe fashion.
 * Can accumulate into a 32bit or 64bit counter.
 * 
 * ThreadSafeCounterClass is expected to be a thread-safe type with a non-static member Add(uint32) that will work correctly if called from multiple threads simultaneously.
 */
template <typename ThreadSafeCounterClass>
class TScopedDurationThreadSafeTimer
{
public:
	explicit TScopedDurationThreadSafeTimer(ThreadSafeCounterClass& InCounter)
		:Counter(InCounter)
		, StartCycles(FPlatformTime::Cycles())
	{
	}
	~TScopedDurationThreadSafeTimer()
	{
		Counter.Add(FPlatformTime::Cycles() - StartCycles);
	}
private:
	ThreadSafeCounterClass& Counter;
	int32 StartCycles;
};

typedef TScopedDurationThreadSafeTimer<FThreadSafeCounter>   FScopedDurationThreadSafeTimer;
typedef TScopedDurationThreadSafeTimer<FThreadSafeCounter64> FScopedDurationThreadSafeTimer64;

/**
 * Utility class for logging the duration of a scoped action (the user 
 * doesn't have to call Start() and Stop() manually).
 */
class FScopedDurationTimeLogger
{
public:
	explicit FScopedDurationTimeLogger(FString InMsg = TEXT("Scoped action"), FOutputDevice* InDevice = GLog)
		: Msg        (MoveTemp(InMsg))
		, Device     (InDevice)
		, Accumulator(0.0)
		, Timer      (Accumulator)
	{
		Timer.Start();
	}

	~FScopedDurationTimeLogger()
	{
		Timer.Stop();
		Device->Logf(TEXT("%s: %f secs"), *Msg, Accumulator);
	}

private:
	FString        Msg;
	FOutputDevice* Device;
	double         Accumulator;
	FDurationTimer Timer;
};

/**
* Utility stopwatch class for tracking the duration of some action (tracks
* time in seconds and adds it to the specified variable on destruction).
* useful for timing that only wants to occur when a feature is optionally turned on 
*/
class FScopedSwitchedDurationTimer
{
public:
	explicit FScopedSwitchedDurationTimer(double& AccumulatorIn, bool bDoFunctionalityIn)
		: StartTime(bDoFunctionalityIn ? FPlatformTime::Seconds() : 0)
		, Accumulator(AccumulatorIn)
		, bDoFunctionality(bDoFunctionalityIn)
	{
	}

	~FScopedSwitchedDurationTimer()
	{
		if (bDoFunctionality)
		{
			Accumulator += (FPlatformTime::Seconds() - StartTime);
		}
	}

	double Start()
	{
		StartTime = FPlatformTime::Seconds();
		return StartTime;
	}

protected:
	/** Start time, captured in ctor. */
	double StartTime;
	/** Time variable to update. */
	double& Accumulator;
	const bool bDoFunctionality;
};

/**
* Utility stopwatch class for tracking the duration of some action (tracks
* time in seconds and adds it to the specified variable on destruction).
* useful for timing that only wants to occur when a feature is optionally turned on
* Also counts the number of timings
*/
class FScopedSwitchedCountedDurationTimer : FScopedSwitchedDurationTimer
{
public:
	explicit FScopedSwitchedCountedDurationTimer(double& TimeAccumulatorIn, int32& CountAccumlatorIn, bool bDoFunctionalityIn)
		: FScopedSwitchedDurationTimer(TimeAccumulatorIn, bDoFunctionalityIn)
	{
		if (bDoFunctionalityIn)
		{
			++CountAccumlatorIn;
		}
	}
};

/**
 * Utility class for logging the duration of a scoped action (the user 
 * doesn't have to call Start() and Stop() manually) using a custom
 * output function.
 */
template<class Func>
class FScopedDurationTimeCustomLogger
{
public:
	explicit FScopedDurationTimeCustomLogger(const TCHAR* InTitle, double& InTotalTime, Func InLogFunc)
		: Title(InTitle)
		, LogFunc(InLogFunc)
		, TotalTime(InTotalTime)
	{
		LogFunc(*FString::Printf(TEXT("%s started..."), InTitle));

		LocalTime = -FPlatformTime::Seconds();
	}

	~FScopedDurationTimeCustomLogger()
	{
		LocalTime += FPlatformTime::Seconds();
		TotalTime += LocalTime;

		TStringBuilder<1024> Msg;
		Msg.Appendf(TEXT("%s took %s"), *Title, *FPlatformTime::PrettyTime(LocalTime));

		if (TotalTime > LocalTime)
		{
			Msg.Appendf(TEXT(" (total: %s)"), *FPlatformTime::PrettyTime(TotalTime));
		}

		LogFunc(*Msg);
	}

private:
	FString Title;
	Func LogFunc;
	double LocalTime;
	double& TotalTime;
};


inline double AtomicDoubleFetchAdd(std::atomic<double>& Value, double Delta,
	std::memory_order MemoryOrder)
{
	double Expected = Value.load(std::memory_order_relaxed);
	double Desired;
	std::memory_order ReadModifyWriteMemoryOrder =
		MemoryOrder == std::memory_order_relaxed ? std::memory_order_relaxed
		: MemoryOrder == std::memory_order_acq_rel ? std::memory_order_acq_rel
		: std::memory_order_seq_cst;

	bool bExchangeSucceeded = false;
	do
	{
		Desired = Expected + Delta;
		bExchangeSucceeded = Value.compare_exchange_weak(Expected, Desired,
			/* success order */ ReadModifyWriteMemoryOrder,
			/* failure order */ std::memory_order_relaxed);
	} while (!bExchangeSucceeded);

	return Expected;
}

#if NO_LOGGING
#define UE_SCOPED_TIMER(Title, Category, Verbosity)
#else
#define UE_SCOPED_TIMER(Title, Category, Verbosity) \
	static double UE_SCOPED_TIMER_COMBINE(ScopedTimerTotal_,__LINE__) = 0.0; FScopedDurationTimeCustomLogger UE_SCOPED_TIMER_COMBINE(ScopedTimer_,__LINE__)(Title, UE_SCOPED_TIMER_COMBINE(ScopedTimerTotal_,__LINE__), [](const TCHAR* Msg) { UE_LOGF(Category, Verbosity, "%ls", Msg) })

#define UE_SCOPED_TIMER_COMBINE_INNER(A,B) A##B
#define UE_SCOPED_TIMER_COMBINE(A,B) UE_SCOPED_TIMER_COMBINE_INNER(A,B)
#endif
