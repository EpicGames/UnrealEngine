// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Math/Vector.h"
#include "Misc/TVariant.h"
#include "Trace/Trace.h"

#ifndef UE_TRACE_SPATIAL_ENABLED
#define UE_TRACE_SPATIAL_ENABLED UE_TRACE_MINIMAL_ENABLED
#endif // UE_TRACE_SPATIAL_ENABLED

#if UE_TRACE_SPATIAL_ENABLED

UE_TRACE_MINIMAL_CHANNEL_EXTERN(SpatialChannel, CORE_API);

namespace UE::SpatialTrace::Private
{
	// Utility class containing either a dynamically allocated string it owns, or a reference to a string literal.
	class FStringOrView
	{
	public:
		using StringType = FString;
		using StringViewType = FStringView;

		explicit FStringOrView(const StringViewType& InView)
			: Storage(TInPlaceType<StringViewType>(), InView)
		{}
		explicit FStringOrView(const StringType& InString)
			: Storage(TInPlaceType<StringType>(), InString)
		{}
		explicit FStringOrView(StringType&& InString)
			: Storage(TInPlaceType<StringType>(), MoveTemp(InString))
		{}

		static FStringOrView FromView(const StringViewType& InView)
		{
			return FStringOrView(InView);
		}

		static FStringOrView FromString(const StringType& InString)
		{
			return FStringOrView(InString);
		}

		static FStringOrView FromString(StringType&& InString)
		{
			return FStringOrView(MoveTemp(InString));
		}

		StringViewType GetView() const
		{
			return Visit([](const auto& Value) -> StringViewType
			{
				return StringViewType(Value);
			}, Storage);
		}

	private:
		TVariant<StringViewType, StringType> Storage;
	};

	enum class EPointComponentFlags : uint8
	{
		None = 0,
		Position = 1 << 0,
		Direction = 1 << 1,
		Velocity = 1 << 2,
		All = 0xFF
	};
	ENUM_CLASS_FLAGS(EPointComponentFlags);

	inline static constexpr uint32 InvalidPointSpecId = 0;

	// Data view structure
	struct FPointDataView
	{
		const FVector* Position = nullptr;
		const FVector* Velocity = nullptr;
		const FVector* Direction = nullptr;
	};

	CORE_API uint32 TracePointSpec(FStringView Name, EPointComponentFlags ComponentMask);
	CORE_API void TracePointSet(uint32 SpecId, EPointComponentFlags SpecComponentMask, EPointComponentFlags ChangedComponentMask, const FPointDataView& ComponentData);
	CORE_API TTuple<uint32, bool> GetOrCreatePointSpecId(std::atomic<uint32>& InOutSpecId, FStringView Name, EPointComponentFlags ComponentMask);

	template <EPointComponentFlags InComponentMask>
	class TPointSpec
	{
		static constexpr EPointComponentFlags ComponentMask = InComponentMask;
		static_assert(EnumHasAnyFlags(ComponentMask, EPointComponentFlags::Position), "Point spec must have Position component.");

		struct FData
		{
			FVector Position;

			struct FEmpty {};
			UE_NO_UNIQUE_ADDRESS std::conditional_t<EnumHasAnyFlags(ComponentMask, EPointComponentFlags::Velocity), FVector, FEmpty> Velocity;
			UE_NO_UNIQUE_ADDRESS std::conditional_t<EnumHasAnyFlags(ComponentMask, EPointComponentFlags::Direction), FVector, FEmpty> Direction;
		};

		FStringOrView Name;
		std::atomic<uint32> Id = InvalidPointSpecId;
		FData Data;

		explicit TPointSpec(FStringOrView&& InName)
			: Name(MoveTemp(InName))
		{
		}

	public:

		TPointSpec(TPointSpec&& Other)
			: Name(MoveTemp(Other.Name))
			, Id(Other.Id.exchange(InvalidPointSpecId))
			, Data(MoveTemp(Other.Data))
		{
		}

		TPointSpec& operator=(TPointSpec&& Other)
		{
			if (this != &Other)
			{
				Name = MoveTemp(Other.Name);
				Id = Other.Id.exchange(InvalidPointSpecId);
				Data = MoveTemp(Other.Data);
			}
			return *this;
		}

		template<typename LiteralType>
		[[nodiscard]] static TPointSpec FromLiteral(const LiteralType& InName)
		{
			static_assert(TIsArrayOrRefOfTypeByPredicate<LiteralType, TIsCharEncodingCompatibleWithTCHAR>::Value, "DisplayName must be a TCHAR array.");
			return TPointSpec(FStringOrView::FromView(InName));
		}

		[[nodiscard]] static TPointSpec FromView(const FStringView& InName)
		{
			return TPointSpec(FStringOrView::FromView(InName));
		}

		[[nodiscard]] static TPointSpec FromString(const FString& InName)
		{
			return TPointSpec(FStringOrView::FromString(InName));
		}

		[[nodiscard]] static TPointSpec FromString(FString&& InName)
		{
			return TPointSpec(FStringOrView::FromString(MoveTemp(InName)));
		}

		void Set(const FVector& Position)
			requires(ComponentMask == EPointComponentFlags::Position)
		{
			SetInternal({
				.Position = Position,
			});
		}

		void Set(const FVector& Position, const FVector& Velocity)
			requires(ComponentMask == (EPointComponentFlags::Position | EPointComponentFlags::Velocity))
		{
			SetInternal({
				.Position = Position,
				.Velocity = Velocity,
			});
		}

		void Set(const FVector& Position, const FVector& Direction)
			requires(ComponentMask == (EPointComponentFlags::Position | EPointComponentFlags::Direction))
		{
			SetInternal({
				.Position = Position,
				.Direction = Direction,
			});
		}

		void Set(const FVector& Position, const FVector& Velocity, const FVector& Direction)
			requires(ComponentMask == (EPointComponentFlags::Position | EPointComponentFlags::Velocity | EPointComponentFlags::Direction))
		{
			SetInternal({
				.Position = Position,
				.Velocity = Velocity,
				.Direction = Direction,
			});
		}

	private:

		void SetInternal(const FData& NewData)
		{
			EPointComponentFlags ChangedMask = EPointComponentFlags::None;
			if constexpr (EnumHasAnyFlags(ComponentMask, EPointComponentFlags::Position))
			{
				if (Data.Position != NewData.Position)
				{
					Data.Position = NewData.Position;
					ChangedMask |= EPointComponentFlags::Position;
				}
			}
			if constexpr (EnumHasAnyFlags(ComponentMask, EPointComponentFlags::Velocity))
			{
				if (Data.Velocity != NewData.Velocity)
				{
					Data.Velocity = NewData.Velocity;
					ChangedMask |= EPointComponentFlags::Velocity;
				}
			}
			if constexpr (EnumHasAnyFlags(ComponentMask, EPointComponentFlags::Direction))
			{
				if (Data.Direction != NewData.Direction)
				{
					Data.Direction = NewData.Direction;
					ChangedMask |= EPointComponentFlags::Direction;
				}
			}

			if (!UE_TRACE_CHANNELEXPR_IS_ENABLED(SpatialChannel) || ChangedMask == EPointComponentFlags::None)
			{
				return;
			}

			uint32 SpecId = InvalidPointSpecId;
			bool bSpecCreated = false;
			Tie(SpecId, bSpecCreated) = GetOrCreatePointSpecId(Id, Name.GetView(), ComponentMask);
			TracePointSet(SpecId, ComponentMask, bSpecCreated ? ComponentMask : ChangedMask, GetDataView());
		}

		FPointDataView GetDataView() const
		{
			FPointDataView DataView;
			if constexpr (EnumHasAnyFlags(ComponentMask, EPointComponentFlags::Position))
			{
				DataView.Position = &Data.Position;
			}
			if constexpr (EnumHasAnyFlags(ComponentMask, EPointComponentFlags::Velocity))
			{
				DataView.Velocity = &Data.Velocity;
			}
			if constexpr (EnumHasAnyFlags(ComponentMask, EPointComponentFlags::Direction))
			{
				DataView.Direction = &Data.Direction;
			}
			return DataView;
		}
	};

} // namespace UE::SpatialTrace::Private

namespace UE::SpatialTrace
{
	using FPointPositionSpec                          = Private::TPointSpec<Private::EPointComponentFlags::Position>;
	using FPointPositionVelocitySpec                  = Private::TPointSpec<Private::EPointComponentFlags::Position | Private::EPointComponentFlags::Velocity>;
	using FPointPositionDirectionSpec                 = Private::TPointSpec<Private::EPointComponentFlags::Position | Private::EPointComponentFlags::Direction>;
	using FPointPositionVelocityDirectionSpec         = Private::TPointSpec<Private::EPointComponentFlags::Position | Private::EPointComponentFlags::Velocity  | Private::EPointComponentFlags::Direction>;
} // namespace UE::SpatialTrace

//////////////////////////////////////////////////
// Internal helpers

#define UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_VAR(Suffix) \
	UE_JOIN(__SpatialTracePointSpec, Suffix)

#define UE_PRIVATE_TRACE_SPATIAL_POINT_INLINE_SPEC_DECLARE(SpecType, DisplayName) \
	static SpecType UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_VAR(__LINE__) = SpecType::FromLiteral(DisplayName);

#define UE_PRIVATE_TRACE_SPATIAL_POINT_INLINE_SPEC(SpecType, DisplayName, ...)   \
	UE_PRIVATE_TRACE_SPATIAL_POINT_INLINE_SPEC_DECLARE(SpecType, DisplayName)    \
	UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_VAR(__LINE__).Set(__VA_ARGS__);

#define UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_DECLARE(SpecType, Name, DisplayName) \
	SpecType UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_VAR(Name) = SpecType::FromLiteral(DisplayName)

#define UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_EXTERN(SpecType, Name) \
	extern SpecType UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_VAR(Name)

//////////////////////////////////////////////////
// Runtime log macros

#define UE_TRACE_SPATIAL_POINT_POS_LOG(Spec, Position)                                            UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_VAR(Spec).Set(Position);
#define UE_TRACE_SPATIAL_POINT_POS_VEL_LOG(Spec, Position, Velocity)                              UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_VAR(Spec).Set(Position, Velocity);
#define UE_TRACE_SPATIAL_POINT_POS_DIR_LOG(Spec, Position, Direction)                             UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_VAR(Spec).Set(Position, Direction);
#define UE_TRACE_SPATIAL_POINT_POS_VEL_DIR_LOG(Spec, Position, Velocity, Direction)               UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_VAR(Spec).Set(Position, Velocity, Direction);

//////////////////////////////////////////////////
// Inline declare & log macros

#define UE_TRACE_SPATIAL_POINT_POS_INLINE_LOG(DisplayName, Position)                              UE_PRIVATE_TRACE_SPATIAL_POINT_INLINE_SPEC(::UE::SpatialTrace::FPointPositionSpec,                   DisplayName, Position);
#define UE_TRACE_SPATIAL_POINT_POS_VEL_INLINE_LOG(DisplayName, Position, Velocity)                UE_PRIVATE_TRACE_SPATIAL_POINT_INLINE_SPEC(::UE::SpatialTrace::FPointPositionVelocitySpec,           DisplayName, Position, Velocity);
#define UE_TRACE_SPATIAL_POINT_POS_DIR_INLINE_LOG(DisplayName, Position, Direction)               UE_PRIVATE_TRACE_SPATIAL_POINT_INLINE_SPEC(::UE::SpatialTrace::FPointPositionDirectionSpec,          DisplayName, Position, Direction);
#define UE_TRACE_SPATIAL_POINT_POS_VEL_DIR_INLINE_LOG(DisplayName, Position, Velocity, Direction) UE_PRIVATE_TRACE_SPATIAL_POINT_INLINE_SPEC(::UE::SpatialTrace::FPointPositionVelocityDirectionSpec,  DisplayName, Position, Velocity, Direction);

//////////////////////////////////////////////////
// Spec‐object declarations

#define UE_TRACE_SPATIAL_POINT_POS_SPEC_DECLARE(Name, DisplayName)         UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_DECLARE(::UE::SpatialTrace::FPointPositionSpec,                  Name, DisplayName);
#define UE_TRACE_SPATIAL_POINT_POS_SPEC_EXTERN(Name)                       UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_EXTERN(::UE::SpatialTrace::FPointPositionSpec,                   Name);

#define UE_TRACE_SPATIAL_POINT_POS_VEL_SPEC_DECLARE(Name, DisplayName)     UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_DECLARE(::UE::SpatialTrace::FPointPositionVelocitySpec,          Name, DisplayName);
#define UE_TRACE_SPATIAL_POINT_POS_VEL_SPEC_EXTERN(Name)                   UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_EXTERN(::UE::SpatialTrace::FPointPositionVelocitySpec,           Name);

#define UE_TRACE_SPATIAL_POINT_POS_DIR_SPEC_DECLARE(Name, DisplayName)     UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_DECLARE(::UE::SpatialTrace::FPointPositionDirectionSpec,         Name, DisplayName);
#define UE_TRACE_SPATIAL_POINT_POS_DIR_SPEC_EXTERN(Name)                   UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_EXTERN(::UE::SpatialTrace::FPointPositionDirectionSpec,          Name);

#define UE_TRACE_SPATIAL_POINT_POS_VEL_DIR_SPEC_DECLARE(Name, DisplayName) UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_DECLARE(::UE::SpatialTrace::FPointPositionVelocityDirectionSpec, Name, DisplayName);
#define UE_TRACE_SPATIAL_POINT_POS_VEL_DIR_SPEC_EXTERN(Name)               UE_PRIVATE_TRACE_SPATIAL_POINT_SPEC_EXTERN(::UE::SpatialTrace::FPointPositionVelocityDirectionSpec,  Name);

#else // UE_TRACE_SPATIAL_ENABLED == 0

//–– No-op stubs ––

#define UE_TRACE_SPATIAL_POINT_POS_LOG(...)
#define UE_TRACE_SPATIAL_POINT_POS_VEL_LOG(...)
#define UE_TRACE_SPATIAL_POINT_POS_DIR_LOG(...)
#define UE_TRACE_SPATIAL_POINT_POS_VEL_DIR_LOG(...)

#define UE_TRACE_SPATIAL_POINT_POS_INLINE_LOG(...)
#define UE_TRACE_SPATIAL_POINT_POS_VEL_INLINE_LOG(...)
#define UE_TRACE_SPATIAL_POINT_POS_DIR_INLINE_LOG(...)
#define UE_TRACE_SPATIAL_POINT_POS_VEL_DIR_INLINE_LOG(...)

#define UE_TRACE_SPATIAL_POINT_POS_SPEC_DECLARE(...)
#define UE_TRACE_SPATIAL_POINT_POS_SPEC_EXTERN(...)
#define UE_TRACE_SPATIAL_POINT_POS_VEL_SPEC_DECLARE(...)
#define UE_TRACE_SPATIAL_POINT_POS_VEL_SPEC_EXTERN(...)
#define UE_TRACE_SPATIAL_POINT_POS_DIR_SPEC_DECLARE(...)
#define UE_TRACE_SPATIAL_POINT_POS_DIR_SPEC_EXTERN(...)
#define UE_TRACE_SPATIAL_POINT_POS_VEL_DIR_SPEC_DECLARE(...)
#define UE_TRACE_SPATIAL_POINT_POS_VEL_DIR_SPEC_EXTERN(...)

#endif // UE_TRACE_SPATIAL_ENABLED
