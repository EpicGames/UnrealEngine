// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "HAL/CriticalSection.h"
#include "HAL/PreprocessorHelpers.h"
#include "Logging/LogMacros.h"
#include "Misc/Build.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/CountersTrace.h"

#include <atomic>

#ifndef STALL_DETECTOR
 #if WITH_EDITOR && ( PLATFORM_WINDOWS || PLATFORM_UNIX ) && !UE_BUILD_SHIPPING && COUNTERSTRACE_ENABLED && !USING_THREAD_SANITISER && !USING_INSTRUMENTATION
  #define STALL_DETECTOR 1
 #else
  #define STALL_DETECTOR 0
 #endif
#endif // STALL_DETECTOR

#if STALL_DETECTOR

/**
 * Counts, but does not report, a stall.
 **/
#define SCOPE_STALL_COUNTER(InName, InBudgetSeconds)                                                                                                                                 \
	static_assert(UE_ARRAY_COUNT(UE_STRINGIZE(InName)) <= UE::StallDetectorScopeMaxNameLength, "Scope name must be less than UE::StallDetectorScopeMaxNameLength characters long "); \
	static UE::FStallDetectorStats UE_JOIN(StallDetectorStats, __LINE__)(TEXT(#InName), InBudgetSeconds, UE::EStallDetectorReportingMode::Disabled);                                 \
	UE::FStallDetector UE_JOIN(ScopeStallDetector, __LINE__)(UE_JOIN(StallDetectorStats, __LINE__));

/**
 * Counts and sends a report of the first occurence of a stall.
 **/
#define SCOPE_STALL_REPORTER(InName, InBudgetSeconds)                                                                                                                                \
	static_assert(UE_ARRAY_COUNT(UE_STRINGIZE(InName)) <= UE::StallDetectorScopeMaxNameLength, "Scope name must be less than UE::StallDetectorScopeMaxNameLength characters long "); \
	static UE::FStallDetectorStats UE_JOIN(StallDetectorStats, __LINE__)(TEXT(#InName), InBudgetSeconds, UE::EStallDetectorReportingMode::First);                                    \
	UE::FStallDetector UE_JOIN(ScopeStallDetector, __LINE__)(UE_JOIN(StallDetectorStats, __LINE__));

/**
 * Counts and reports every occurence of a stall.
 * Use with extreme caution as you could overload the report backend!
 * This mainly exists to help test and exercise the stall detector system.
 **/
#define SCOPE_STALL_REPORTER_ALWAYS(InName, InBudgetSeconds)                                                                                                                         \
	static_assert(UE_ARRAY_COUNT(UE_STRINGIZE(InName)) <= UE::StallDetectorScopeMaxNameLength, "Scope name must be less than UE::StallDetectorScopeMaxNameLength characters long "); \
	static UE::FStallDetectorStats UE_JOIN(StallDetectorStats, __LINE__)(TEXT(#InName), InBudgetSeconds, UE::EStallDetectorReportingMode::Always);                                   \
	UE::FStallDetector UE_JOIN(ScopeStallDetector, __LINE__)(UE_JOIN(StallDetectorStats, __LINE__));

/**
 * Pauses stall detectors on the current thread so that time in the current thread spent in this scope will not be counted as a stall.
 **/
#define SCOPE_STALL_DETECTOR_PAUSE() \
	UE::FStallDetectorPause UE_JOIN(ScopeStallDetectorPause, __LINE__);

#else // STALL_DETECTOR

#define SCOPE_STALL_COUNTER(InName, InBudgetSeconds)
#define SCOPE_STALL_REPORTER(InName, InBudgetSeconds)
#define SCOPE_STALL_REPORTER_ALWAYS(InName, InBudgetSeconds)
#define SCOPE_STALL_DETECTOR_PAUSE()

#endif // STALL_DETECTOR

#if STALL_DETECTOR

CORE_API DECLARE_LOG_CATEGORY_EXTERN(LogStall, Log, All);

/**
 * This code is meant to instrument code to send reports for slow pathways that create unresponsive conditions
 **/
namespace UE
{
	class FStallDetector;

	// Stall detected event is fired on a worker thread when it is detected that a stall scope has been active for too long.
	// Fired from a worker thread.
	struct FStallDetectedParams
	{
		/** Name of Stall Detector instance*/
		FStringView StatName;
		/** TID of stalled thread */
		uint32 ThreadId;
		/** UniqueID of the stalled instance */
		uint64 UniqueID;
		/** Backtrace of stall */
		TArray<uint64> Backtrace;
	};
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnStallDetected, const FStallDetectedParams&);

	// Stall completed event is fired synchronously on the stalling thread after it recovers from the stall.
	// It may be fired without a matching StallDetected event in the case that the background detector's resolution
	// is too low or that thread is blocked for some reason.
	struct FStallCompletedParams
	{
		/** TID of stalled thread */
		uint32 ThreadId;
		/** Name of Stall Detector instance*/
		FStringView StatName;
		/** Budget for this scope. */
		double BudgetSeconds;
		/** Amount of time StallDetector instance stalled beyond the budget. */
		double OverbudgetSeconds;
		/** UniqueID of the stalled instance */
		uint64 UniqueID;
		/** 
		 * True if the background thread detected a stall in this scope.
		 * Since the compelted delegate is fired from the thread containing the scope, 
		 * OnStallDetected and OnStallCompleted are fired on different threads and may be executed in any order.
		 * If bWasTriggered is true, then OnStallDetected has fired or will fire for this UniqueID.
		 **/
		bool bWasTriggered;
	};
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnStallCompleted, const FStallCompletedParams&);

	/**
	* The reporting behavior of the detector
	**/
	enum EStallDetectorReportingMode
	{
		Disabled,
		First,
		Always
	};

	static constexpr SIZE_T StallDetectorStatNameBufferSize = 256;
	static constexpr int32 StallDetectorMaxBacktraceEntries = 32;

	class FStallDetectorRunnable;

	/**
	 * This structure is allocated as a function-local static to persist stats across stall checks.
	 */
	class FStallDetectorStats
	{
		friend class FStallDetectorRunnable;

	public:
		/**
		* Construct a stats object for use with a stall detector
		*/
		CORE_API FStallDetectorStats(const TCHAR* InName, const double InBudgetSeconds, const EStallDetectorReportingMode InReportingMode);

		/**
		* Destruct a stats object, just removes it from the instance tracking
		*/
		CORE_API ~FStallDetectorStats();

		/**
		* Stall detector scope has ended, record the interval
		*/
		CORE_API void OnStallCompleted(double InOverageSeconds);

		struct TabulatedResult
		{
			const TCHAR* Name = TEXT("");
			double BudgetSeconds = 0.0;
			int64 TriggerCount = 0;
			double OverageSeconds = 0.0;
			double OverageRatio = 0.0;
		};
		static CORE_API void TabulateStats(TArray<TabulatedResult>& TabulatedResults);

		// The name to refer to this callsite in the codebase
		const TCHAR* Name;

		// The budgeted time we tolerate without sending a report
		const double BudgetSeconds;

		// Drives the behavior about what to do when a budget is violated (triggered)
		const EStallDetectorReportingMode ReportingMode;

		// Has this been reported yet, this is different than the trigger count for coherency reasons
		bool bReported;

		// The total number of times all callsites have been triggered
		static CORE_API FCountersTrace::FCounterAtomicInt TotalTriggeredCount;

		// The total number of reports that have been sent
		static CORE_API FCountersTrace::FCounterAtomicInt TotalReportedCount;

	private:
		// The number of times this callsite has been triggered
		TCHAR TriggerCountCounterName[StallDetectorStatNameBufferSize];
		FCountersTrace::TCounter<int64, TraceCounterType_Int> TriggerCount;

		// The cumulative overage time for this callsite
		TCHAR OverageSecondsCounterName[StallDetectorStatNameBufferSize];
		FCountersTrace::TCounter<double, TraceCounterType_Float> OverageSeconds;

		// Guards access to the stats from multiple threads, for coherency
		mutable FCriticalSection StatsSection;
	};

	/**
	 * FStallTimer tracks remaining time from some budget, and access is protected from multiple threads.
	 * Owned by FStallDetector
	 **/
	class FStallTimer
	{
	public:
		/**
		* Construct the timer, you must call Reset() before the other methods
		*/
		CORE_API FStallTimer();

		/**
		* Initialize the timer with the current timestamp and remaining time to track
		**/
		CORE_API void Reset(const double InSeconds, const double InRemainingSeconds);

		/**
		* Perform a check against the specified timestamp and provide some useful details
		**/
		CORE_API void Check(const double InSeconds, double& OutDeltaSeconds, double& OutOverageSeconds);

		/**
		* Pause the timer
		**/
		CORE_API void Pause(const double InSeconds);

		/**
		* Resume the timer
		**/
		CORE_API void Resume(const double InSeconds);

	private:
		// To track pause/unpause calls
		uint32 PauseCount;

		// The timestamp of when we last called Reset() or Check()
		double LastCheckSeconds;

		// The time remaining before we we go beyond the remaining time
		double RemainingSeconds;

		// Guards access to internal state from multiple threads
		FCriticalSection Section;
	};

	/**
	* FStallDetector is meant to be constructed and destructed across a single scope to measure that specific timespan
	**/
	class FStallDetector
	{
		friend class FStallDetectorRunnable;

	public:
		/**
		* Construct the stall detector linking it to it's stats object and take a timestamp
		*/
		CORE_API FStallDetector(FStallDetectorStats& InStats);

		/**
		* Destruct the stall detector and perform a check if appropriate (the default behavior)
		*/
		CORE_API ~FStallDetector();

		/**
		* Perform a Check and then reset the timer to the current time
		* This changes the mode of this class to defer the first check, and skip the destruction check
		* This mode enables this class to used as a global or static, and CheckAndReset() callable periodically
		**/
		CORE_API void CheckAndReset();

		/**
		* Pause the timer for an expectantly long/slow operation we don't want counted against the stall timer
		**/
		CORE_API void Pause(const double InSeconds);

		/**
		* Resume the timer at the specified timestamp
		**/
		CORE_API void Resume(const double InSeconds);

		/**
		* The Parent stall detector on the calling thread's stack space
		**/
		FStallDetector* GetParent()
		{
			return Parent;
		}

		/**
		* Sample the same clock that the stall detector uses internally
		**/
		static CORE_API double Seconds();

		/**
		* Initialize Stall Detector API resources
		**/
		static CORE_API void Startup();

		/**
		* Cleanup Stall Detector API resources
		**/
		static CORE_API void Shutdown();

		/**
		* Stall Detector API is running?
		**/
		static CORE_API bool IsRunning();

		/**
		 * Event that triggers when we detect that a stall in any instance is occuring
		 * Note: It is possible for multiple instances to stall at the same time,
		 * delegates should take this into account 
		 */
		static CORE_API FOnStallDetected StallDetected;

		/**
		 * Event that triggers when a stall has stopped
		 * Note: It is possible for multiple instances to complete their stall at the same time,
		 * delegates should take this into account
		 */
		static CORE_API FOnStallCompleted StallCompleted;

	private:
		/** Monotonically increasing counter to generate a unique ID per stall detector */
		static CORE_API TAtomic<uint64> UIDGenerator;
		
		// The time provided, the deadline
		FStallDetectorStats& Stats;

		// A stall detector on the same thread with a longer lifetime if any
		FStallDetector* Parent;

		// Thread it was constructed on, and to stack trace when budgeted time is elapsed
		uint32 ThreadId;

		// Did we actually start the detector (was the thread started at construction time)
		bool bStarted;

		// Persistent usage mode
		bool bPersistent;

		// Track whether a stall was detected from the background thread.
		// This is protected by the same mutex which protects the registered stall detector instances.
		// Only set from background thread, can be reset from another thread with CheckAndReset
		bool bTriggered;

		// The timer data for this detector
		FStallTimer Timer;

		// Unique ID given to a StallDetector
		uint64 UniqueID;

		/**
		 * Check the detector to see if we are over budget, called from both construction and background thread
		 * @param bFinalCheck			Is this check the one performed at completion of the scope or measurement period?
		 * @param InCheckSeconds		The timestamp at which to compare the start time to
		 * @return The amount the scope's budget has been exceeded by or 0.0
		 **/
		CORE_API double Check(bool bFinalCheck, double InCheckSeconds);
	};

	/**
	* FStallDetectorPause checks, pauses, and resumes the current thread's stack of stall detectors across it's lifetime
	**/
	class FStallDetectorPause
	{
	public:
		CORE_API FStallDetectorPause();
		CORE_API ~FStallDetectorPause();
		bool IsPaused() const;

	private:
		bool bPaused;
	};

	static constexpr TCHAR StallDetectorCounterPrefix[] = TEXT("StallDetector/");
	static constexpr TCHAR StallDetectorTriggerCountSuffix[] = TEXT(" TriggerCount");
	static constexpr TCHAR StallDetectorOverageTimeSuffix[] = TEXT(" OverageSeconds");
	static constexpr SIZE_T StallDetectorScopeMaxNameLength = StallDetectorStatNameBufferSize - 1 - UE_ARRAY_COUNT(StallDetectorCounterPrefix)
															+ FMath::Max(UE_ARRAY_COUNT(StallDetectorTriggerCountSuffix), UE_ARRAY_COUNT(StallDetectorOverageTimeSuffix));
} // namespace UE
#endif // STALL_DETECTOR
