// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"

CORE_API UE_TRACE_EVENT_BEGIN_EXTERN(Strings, StaticString, NoSync|Definition64bit)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, DisplayWide)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, DisplayAnsi)
UE_TRACE_EVENT_END()

CORE_API UE_TRACE_EVENT_BEGIN_EXTERN(Strings, FName, NoSync|Definition32bit)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, DisplayWide)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, DisplayAnsi)
UE_TRACE_EVENT_END()

#if !UE_TRACE_ENABLED
#define UE_API FORCEINLINE
#else
#define UE_API CORE_API
#endif

/**
 * Utility class to trace deduplicated strings, FNames and static strings. Each function returning a
 * reference that can be used when tracing events.
 *
 * A event defined as:
 * \code
 * UE_TRACE_EVENT_BEGIN(Asset)
 *		UE_TRACE_EVENT_REFERENCE_FIELD(Strings, FName, Name)
 *		UE_TRACE_EVENT_REFERENCE_FIELD(Strings, StaticString, File)
 * UE_TRACE_EVENT_END()
 *
 * Can be emitted with:
 * auto NameRef = FStringTrace::GetNameRef(Object->GetFName());
 * auto FileRef = FStringTrace::GetStaticStringRef(__FILE__);
 * UE_TRACE_LOG(Asset)
 *		<< Asset.Name(NameRef)
 *		<< Asset.File(FileRef);
 * \endcode
 */
class FStringTrace
{
	
public:
	/**
	 * Gets the trace id of a FName.
	 * @param Name Name to trace.
	 * @return Id that can be used to reference the name.
	 */
	UE_API static UE::Trace::FEventRef32 GetNameRef(const FName& Name);

	/**
	 * Gets the trace id of a static string. Will use the address of the string
	 * for deduplication, so it is important to only pass static strings to this
	 * function.
	 * @param String Static string
	 * @return Id that can be used to reference the string.
	 */
	UE_API static UE::Trace::FEventRef64 GetStaticStringRef(const TCHAR* String);
	
	/**
	 * Gets the trace id of a static string. Will use the address of the string
	 * for deduplication, so it is important to only pass static strings to this
	 * function.
	 * @param String Static string
	 * @return Id that can be used to reference the string.
	 */
	UE_API static UE::Trace::FEventRef64 GetStaticStringRef(const ANSICHAR* String);

	/**
	 * On a new connection callback.
	 */
	static void OnConnection();
};

#if !UE_TRACE_ENABLED
UE_FORCEINLINE_HINT UE::Trace::FEventRef32 FStringTrace::GetNameRef(const FName& Name) { return UE::Trace::FEventRef32(0,0); }
UE_FORCEINLINE_HINT UE::Trace::FEventRef64 FStringTrace::GetStaticStringRef(const TCHAR* String) { return UE::Trace::FEventRef64(0,0); }
UE_FORCEINLINE_HINT UE::Trace::FEventRef64 FStringTrace::GetStaticStringRef(const ANSICHAR* String) { return UE::Trace::FEventRef64(0,0); }
UE_FORCEINLINE_HINT void FStringTrace::OnConnection() {};
#endif

#undef UE_API