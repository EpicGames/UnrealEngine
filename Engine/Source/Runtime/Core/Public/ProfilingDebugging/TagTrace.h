// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "Trace/Config.h"
#include "Trace/Detail/LogScope.h"

class FName;

////////////////////////////////////////////////////////////////////////////////
// Fwd declare ELLMTag
enum class ELLMTag : uint8;
// Fwd declare LLM private tag data
namespace UE {
	namespace LLMPrivate {
		class FTagData;
	}
}

////////////////////////////////////////////////////////////////////////////////
CORE_API int32	MemoryTrace_AnnounceCustomTag(int32 Tag, int32 ParentTag, const TCHAR* Display);
CORE_API int32	MemoryTrace_AnnounceFNameTag(const class FName& TagName);
CORE_API int32	MemoryTrace_GetActiveTag();

inline constexpr int32 TRACE_TAG = 257;

////////////////////////////////////////////////////////////////////////////////
#if !defined(UE_MEMORY_TAGS_TRACE_ENABLED)
	#define UE_MEMORY_TAGS_TRACE_ENABLED 0
#endif

#if UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED

////////////////////////////////////////////////////////////////////////////////

/**
  * Used to associate any allocation within this scope to a given tag.
  *
  * We need to be able to convert the three types of inputs to LLM scopes:
  * - ELLMTag, an uint8 with fixed categories. There are three sub ranges
	  Generic tags, platform and project tags.
  * - FName, free form string, for example a specific asset.
  * - TagData, an opaque pointer from LLM.
  *
  */
class FMemScope
{
public:
	CORE_API FMemScope(); // Used with SetTagAndActivate
	CORE_API FMemScope(int32 InTag, bool bShouldActivate = true);
	CORE_API FMemScope(ELLMTag InTag, bool bShouldActivate = true);
	CORE_API FMemScope(const class FName& InName, bool bShouldActivate = true);
	CORE_API FMemScope(const UE::LLMPrivate::FTagData* TagData, bool bShouldActivate = true);
	CORE_API ~FMemScope();

	CORE_API void SetTagAndActivate(const class FName& InName, bool bShouldActivate = true);

private:
	void ActivateScope(int32 InTag);
	UE::Trace::Private::FScopedLogScope Inner;
	int32 PrevTag;
};


/**
 * A scope that activates in case no existing scope is active.
 */
template<typename TagType>
class FDefaultMemScope : public FMemScope
{
public:
	FDefaultMemScope(TagType InTag)
		: FMemScope(InTag, MemoryTrace_GetActiveTag() == 0)
	{
	}
};

/**
 * Used order to keep the tag for memory that is being reallocated.
 */
class FMemScopePtr
{
public:
	CORE_API FMemScopePtr(uint64 InPtr, HeapId InRootHeap = EMemoryTraceRootHeap::SystemMemory);
	CORE_API ~FMemScopePtr();
private:
	UE::Trace::Private::FScopedLogScope Inner;
};

////////////////////////////////////////////////////////////////////////////////
#define UE_MEMSCOPE(InTag)					FMemScope UE_JOIN(MemScope,__LINE__)(InTag);
// UE_MEMSCOPE_PTR(InPtr) or UE_MEMSCOPE_PTR(InPtr, InRootHeap)
#define UE_MEMSCOPE_PTR(InPtr, ...)			FMemScopePtr UE_JOIN(MemPtrScope,__LINE__)((uint64)InPtr, ##__VA_ARGS__);
#define UE_MEMSCOPE_DEFAULT(InTag)			FDefaultMemScope UE_JOIN(MemScope,__LINE__)(InTag);
#define UE_MEMSCOPE_UNINITIALIZED(Line)		FMemScope UE_JOIN(MemScope,Line);
#define UE_MEMSCOPE_ACTIVATE(Line, InTag)	UE_JOIN(MemScope,Line).SetTagAndActivate(InTag);

#else // UE_MEMORY_TAGS_TRACE_ENABLED

////////////////////////////////////////////////////////////////////////////////
#define UE_MEMSCOPE(...)
#define UE_MEMSCOPE_PTR(...)
#define UE_MEMSCOPE_DEFAULT(...)
#define UE_MEMSCOPE_UNINITIALIZED(...)
#define UE_MEMSCOPE_ACTIVATE(...)

#endif // UE_MEMORY_TAGS_TRACE_ENABLED

