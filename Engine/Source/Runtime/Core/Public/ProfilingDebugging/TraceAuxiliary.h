// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/AnsiString.h"
#include "Containers/StringFwd.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/Build.h"
#include "Misc/CoreMiscDefines.h"

#if !defined(UE_TRACE_SERVER_CONTROLS_ENABLED)
#	define UE_TRACE_SERVER_CONTROLS_ENABLED (PLATFORM_DESKTOP && !UE_BUILD_SHIPPING && !IS_PROGRAM)
#endif

////////////////////////////////////////////////////////////////////////////////
class FTraceAuxiliary
{
public:
	/**
	* This enum is serialized and sent via the trace service.
	* Do not change the values or modify the order. Only add new values to the end.
	* Should be kept in sync with FTraceStatus::ETraceSystemStatus from ITraceController.h
	*/
	enum class ETraceSystemStatus : uint8
	{
		NotAvailable, // Disabled at compile time.
		Available,
		TracingToServer,
		TracingToFile,
		TracingToSecureNetwork,
		TracingToCustomRelay,

		NumValues, // This must be the last value.
	};

	enum class EEnumerateResult : uint8
	{
		Continue,
		Stop,
	};

	struct FChannelPreset
	{
		FChannelPreset(const TCHAR* InName,const TCHAR* InChannels, bool bInIsReadOnly)
			: Name(InName)
			, ChannelList(InChannels)
			, bIsReadOnly(bInIsReadOnly)
		{
		}

		/**
		 * Do not store these pointers.
		 */
		const TCHAR* Name;
		const TCHAR* ChannelList;

		/**
		* A preset should be read-only if it contains any read-only channels.
		* A read-only preset can only be enabled using the command line when starting the application.
		*/
		bool bIsReadOnly = false;
	};

	typedef TFunctionRef<EEnumerateResult(const FChannelPreset& Preset)> PresetCallback;

	// In no logging configurations all log categories are of type FNoLoggingCategory, which has no relation with
	// FLogCategoryBase. In order to not need to conditionally set the argument alias the type here.
#if NO_LOGGING
	typedef FNoLoggingCategory FLogCategoryAlias;
#else
	typedef FLogCategoryBase FLogCategoryAlias;
#endif

	enum class EConnectionType : uint8
	{
		/**
		 * Connect to a trace server. Target is IP address or hostname.
		 */
		Network,
		/**
		 * Write to a file. Target string is filename. Absolute or relative current working directory.
		 * If target is null the current date and time is used.
		 */
		File,
		/**
		 * Relay connection. Pass user defined I/O functions to write raw trace data.
		 */
		Relay,
		/**
		 * Connect to a receiver using a secure socket connection.
		 */
		SecureNetwork,
		/**
		 * Don't connect, just start tracing to memory.
		 */
		None,
	};

	/**
	 * Callback type when a new connection is established.
	 */
	DECLARE_TS_MULTICAST_DELEGATE(FOnConnection);

	/**
	 * Callback when a connection is closed.
	 */
	DECLARE_TS_MULTICAST_DELEGATE(FOnClosed);

	/** Callback whenever a trace is started */
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnTraceStarted, FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination);

	/**
	 * Callback whenever a trace recording is stopped.
	 * @param TraceType Kind of trace
	 * @param TraceDestination Either filename and path for a file trace or the network connection for a network trace
	 * @note Callback will not trigger if trace is stopped by TraceLog itself, for example network interuptions or disk write errors.
	 */
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnTraceStopped, FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination);

	/**
	 * Callback whenever a trace snapshot is saved.
	 * Path is the file system path of the snapshot file.
	 */
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnSnapshotSaved, FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination);

	struct FOptions
	{
		/** When set, trace will not start a worker thread, instead it is updated from end frame delegate. */
		UE_DEPRECATED(5.7, "Use -notracethread command line instead")
		bool bNoWorkerThread = false;
		/** When set, the target file will be truncated if it already exists. */
		bool bTruncateFile = false;
		/** When set, trace data buffered before starting tracing will not be output to the trace file */
		bool bExcludeTail = false;
	};

	/**
	 * Start tracing to a target (network connection or file) with an active set of channels. If a connection is
	 * already active this call does nothing.
	 * @param Type Type of connection. Network or File type. If a custom target is desired use \ref FTraceAuxiliary::Relay.
	 * @param Target String to use for connection. See /ref EConnectionType for details.
	 * @param Channels Comma separated list of channels to enable. Default set of channels are enabled if argument is not specified. If the pointer is null no channels are enabled.
	 * @param Options Optional additional tracing options.
	 * @param LogCategory Log channel to output messages to. Default set to 'Core'.
	 * @return True when successfully starting the trace, false if the data connection could not be made.
	 */
	static CORE_API bool Start(EConnectionType Type, const TCHAR* Target, const TCHAR* Channels = TEXT("default"), FOptions* Options = nullptr, const FLogCategoryAlias& LogCategory = LogCore);

	/**
	 * Start tracing to a custom target by providing a handle and a write and close function. If a connection is already active this call does nothing.
	 * @param Handle Anonymous handle to trace to. This will be passed to the writer and close functions.
	 * @param WriteFunc A function to handle writing of trace data
	 * @param CloseFunc A function to handle closing of the data stream.
	 * @param Channels Comma separated list of channels to enable. Default set of channels are enabled if argument is not specified. If the pointer is null no channels are enabled.
	 * @param Options Optional additional tracing options.
	 * @return True when successfully starting the trace, false if the data connection could not be made.
	 */
	static CORE_API bool Relay(UPTRINT Handle, UE::Trace::IoWriteFunc WriteFunc, UE::Trace::IoCloseFunc CloseFunc, const TCHAR* Channels = TEXT("default"), const FOptions* Options = nullptr);


#if UE_TRACE_ALLOW_SECURE_TRACING

	enum class ESecureMode : uint8
	{
		/**
		 * Creates a self signed certficate, accessible via /ref GetSecureCertificate. Connecting peer is 
		 * expected to send the authorization token before data will be received.
		 */
		SelfSignedCertAndAuthToken,
		/**
		 * Use a externally generated certificate and private key. Connecting peer is expected to have 
		 * loaded the same certificate. Connecting peer is expected to send the authorization token 
		 * before data will be received.
		 */
		CertFileAndAuthToken,
		/**
		 * Use authorization token as Pre-Shared Key verification.
		 */
		AuthToken
	};

	struct FSecureOptions 
	{
		// Specifies identity of the peer expected to connect if mode is AuthToken
		FAnsiString Identity;
		// Authorization token
		TArray<uint8> AuthorizationToken;
		// Timeout for connecting steps. A zero value means no timeouts.
		float TimeoutSeconds = 0;
	};

	/**
	 * Start tracing to network using a secure socket. This method is different from other tracing methods as it requires
	 * the receiving peer to connect to this machine. Note that some options are controlled at the application level.
	 * @param SecureOptions Options for how the secure connection is made
	 * @param Options Optional additional tracing options.
	 * @param Channels Comma separated list of channels to enable. Default set of channels are enabled if argument is not specified. If the pointer is null no channels are enabled.
	 */
	static CORE_API bool StartSecure(FSecureOptions& SecureOptions, FOptions* Options = nullptr, const TCHAR* Channels = TEXT("default"));

	/**
	 * Retrieve the certificate if secure tracing was initialized with using self-signed certificates.
	 * @return The certificate in DER format if using self-signed certificates. Otherwise an empty view. The view is valid for the remainder of the
	 * process lifetime.
	 */
	static CORE_API TArrayView<uint8> GetSecureCertificate();

	/**
	 * Gets the configured hostname connecting peers needs to use.
	 * @return Hostname or IP
	 */
	static CORE_API FString GetSecureHost();

	/**
	 * Gets the configured port connecting peers should use.
	 * @return Port number secure connection will listen to.
	 */
	static CORE_API uint16 GetSecurePort();
#endif

	/**
	 * Stop tracing.
	 * @return True if the trace was stopped, false if there was no data connection.
	 */
	static CORE_API bool Stop();

	/**
	 * Pause all tracing by disabling all active channels.
	 */
	static CORE_API bool Pause();

	/**
	 * @return True if trace was paused and the list of channels to resume exists.
	 */
	static CORE_API bool IsPaused();

	/**
	 * Resume tracing by enabling all previously active channels.
	 */
	static CORE_API bool Resume();

	/**
	 * Write tailing memory state to a utrace file.
	 * @param FilePath Path to the file to write the snapshot to. If it is null or empty a file path will be generated.
	 * @param MaxTailSize Maximum size of tail buffer to write. Zero means no limit. Total file size may be larger due to important events and
	 * overhead.
	 */
	static CORE_API bool WriteSnapshot(const TCHAR* FilePath, uint32 MaxTailSize = 0);

	/**
	 * Write tailing memory state to a trace server.
	 * @param Host Host to send snapshot to
	 * @param Port Port to connect to host on. Zero means default port is used 
	 * @param MaxTailSize Maximum size of tail buffer to write. Zero means no limit. Total stream size may be larger due to important events and
	 * overhead.
	 */
	static CORE_API bool SendSnapshot(const TCHAR* Host = nullptr, uint32 Port = 0, uint32 MaxTailSize = 0);

	/**
	 * Initialize Trace systems.
	 * @param CommandLine to use for initializing
	 */
	static CORE_API void Initialize(const TCHAR* CommandLine);

	/**
	 * Initialize channels that use the config driven presets.
	 * @param CommandLine to use for initializing
	 */
	static CORE_API void InitializePresets(const TCHAR* CommandLine);

	/**
	 * Shut down Trace systems.
	 */
	static CORE_API void Shutdown();

	/**
	 * Attempts to auto connect to an active trace server if an active session
	 * of Unreal Insights Session Browser is running.
	 */
	static CORE_API void TryAutoConnect();

	/**
	 *  Enable previously selected channels. This method can be called multiple times
	 *  as channels can be announced on module loading.
	 */
	static CORE_API void EnableCommandlineChannels();

	/**
	 * Enable channels to emit events belonging to this category.
	 * @note Note that presets cannot be used when specifying channel ids.
	 * @param ChannelIds List of channels to enable.
	 * @param OutErrors Contains deny reasons for channels that could not be enabled.
	 */
	static CORE_API void EnableChannels(TConstArrayView<uint32> ChannelIds, TMap<uint32, FString>* OutErrors = nullptr);

	/**
	 * Disable channels to mute events belonging to this category.
	 * @note Note that presets cannot be used when specifying channel ids.
	 * @param ChannelIds List of channels to disable.
	 * @param OutErrors Contains deny reasons for channels that could not be disabled.
	 */
	static CORE_API void DisableChannels(TConstArrayView<uint32> ChannelIds, TMap<uint32, FString>* OutErrors = nullptr);

	/**
	 * Enable channels to emit events from this category.
	 * @param Channels Comma separated list of channels and presets to enable.
	 */
	static CORE_API void EnableChannels(const TCHAR* Channels);

	/**
	 * Disable channels to stop recording traces with them.
	 * @param Channels Comma separated list of channels and presets to disable. If null it will disable all active channels.
	 */
	static CORE_API void DisableChannels(const TCHAR* Channels = nullptr);

	/**
	 *  Returns the destination string that is currently being traced to.
	 *  Contains either a file path or network address. Empty if tracing is disabled.
	 */
	static CORE_API FString GetTraceDestinationString();

	/**
	 *  Returns whether the trace system is currently connected to a trace sink (network, file or custom relay).
	 */
	static CORE_API bool IsConnected();

	/**
	 * Returns whether the trace system is currently connected. If connected, it writes the session/trace identifiers.
	 * @param OutSessionGuid If connected, the session guid
	 * @param OutTraceGuid If connected, the trace guid
	 * @return True if connected
	 */
	static CORE_API bool IsConnected(FGuid& OutSessionGuid, FGuid& OutTraceGuid);

	/**
	 * Returns the current connection type.
	 */
	static CORE_API EConnectionType GetConnectionType();

	/**
	 * Adds a comma separated list of currently active channels to the passed in StringBuilder
	 */
	static CORE_API void GetActiveChannelsString(FStringBuilderBase& String);

	/**
	 * Used when process is panicking. Stops all tracing immediately to avoid further allocations. Process is not
	 * expected to continue after this call.
	 */
	static CORE_API void Panic();

	/**
	 * Get the settings used to initialize TraceLog
	 */
	static CORE_API struct UE::Trace::FInitializeDesc const* GetInitializeDesc();

	/**
	* Enumerate the channel presets that are defined in code.
	*/
	static CORE_API void EnumerateFixedChannelPresets(PresetCallback Callback);

	/**
	* Enumerate the channel presets that are defined in BaseEngine.ini, under the [Trace.ChannelPresets] section.
	*/
	static CORE_API void EnumerateChannelPresetsFromSettings(PresetCallback Callback);

	/**
	 * Delegate that triggers when a connection is established. Gives subscribers a chance to trace events that appear
	 * after important events but before regular events (including tail). The following restrictions apply:
	 *  * Only NoSync event types can be emitted.
	 *  * Important events should not be emitted. They will appear after the events in the tail.
	 *  * Callback is issued from a worker thread. User is responsible to synchronize shared resources.
	 *
	 * @note This is an advanced feature to avoid using important events in cases where event data can be recalled easily.
	 *
	 * @param Callback Delegate to call on new connections.
	 */
	static CORE_API FOnConnection OnConnection;

	/**
	 * Delegate that triggers when a connection is closed.
	 */
	static CORE_API FOnClosed OnClosed;

	/**
	 * Delegate that triggers when a trace session is started.
	 * The type of recording and the destination (filepath or network) is passed to the delegate.
	 */
	static CORE_API FOnTraceStarted OnTraceStarted;

	/**
	 * Delegate that triggers when a trace has finished recording. Useful if you need to collect all completed trace files in a session.
	 * The type of recording and the destination (filepath or network) is passed to the delegate.
	 */
	static CORE_API FOnTraceStopped OnTraceStopped;

	/**
	 * Delegate that triggers when a snapshot has been saved.
	 * The path to the snapshot file is passed to the delegate.
	 */
	static CORE_API FOnSnapshotSaved OnSnapshotSaved;

	/**
	 * Returns the current status of the trace system.
	 */
	static CORE_API ETraceSystemStatus GetTraceSystemStatus();
};

#if UE_TRACE_SERVER_CONTROLS_ENABLED

/**
 * Controls for Unreal Trace Server, the standalone server recording and storing traces.
 */
class FTraceServerControls
{
public:
	/**
	 * Launch the server using the "fork" command. This spins off a separate running process.
	 * @return True if the server was successfully started or already running.
	 */
	static CORE_API bool Start();

	/**
	 * Stop any running instance of the server.
	 * @return True if the stop command was successful. False otherwise.
	 */
	static CORE_API bool Stop();
};

#endif // UE_TRACE_SERVER_CONTROLS_ENABLED
