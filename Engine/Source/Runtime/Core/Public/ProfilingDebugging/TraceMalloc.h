// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MemoryTrace.h"
#include "HAL/MemoryBase.h"

class FTraceMalloc : public FMalloc
{
public:
	FTraceMalloc(FMalloc* InMalloc);
	virtual ~FTraceMalloc();

	virtual void* Malloc(SIZE_T Count, uint32 Alignment) override;
	virtual void* Realloc(void* Original, SIZE_T Count, uint32 Alignment) override;
	virtual void Free(void* Original) override;

	virtual void OnMallocInitialized() override
	{
		WrappedMalloc->OnMallocInitialized();
	}

	virtual void OnPreFork() override
	{
		WrappedMalloc->OnPreFork();
	}

	virtual void OnPostFork() override
	{
		WrappedMalloc->OnPostFork();
	}

	FMalloc* WrappedMalloc;
};
