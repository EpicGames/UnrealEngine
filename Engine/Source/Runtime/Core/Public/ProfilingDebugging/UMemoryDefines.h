// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * true if intelTBB (a.k.a MallocTBB) can be used (different to the platform actually supporting it)
 */
#ifndef TBB_ALLOCATOR_ALLOWED
	#define TBB_ALLOCATOR_ALLOWED 1
#endif

 /**
  * true if mimalloc can be used (different to the platform actually supporting it)
  */
#ifndef MIMALLOC_ALLOCATOR_ALLOWED
	#define MIMALLOC_ALLOCATOR_ALLOWED 1
#endif
