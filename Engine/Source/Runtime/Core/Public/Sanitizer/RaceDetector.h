// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

#if USING_INSTRUMENTATION

#include "Instrumentation/Defines.h"
#include "Sanitizer/Types.h"
#include "Misc/Timeout.h"

namespace UE::Sanitizer::RaceDetector
{
	// Setup everything that is required for the race detector.
	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES CORE_API bool Initialize();
	// Cleanup everything about the race detector that was setup in Initialize.
	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES CORE_API bool Shutdown();

	typedef TFunctionRef<void(uint64 RaceAddress, uint32 FirstThreadId, uint32 SecondThreadId, const FFullLocation& FirstLocation, const FFullLocation& SecondLocation)> TRaceCallbackFn;
	// Set a callback that will be called whenever a race is detected (filters will not apply).
	// NOTE: It is not safe to perform any allocation within the callback since it will be
	//       called from instrumentation code.
	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES CORE_API void SetRaceCallbackFn(TRaceCallbackFn CallbackFn);
	// Remove any callback that was previously set.
	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES CORE_API void ResetRaceCallbackFn();

	// Dump the current thread context to help with debugging.
	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES CORE_API void DumpContext();
	// Dump the current thread context but with additional info.
	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES CORE_API  void DumpContextDetailed();

	// Means that races that doesn't involve the current thread will be filtered.
	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES CORE_API void ToggleFilterOtherThreads(bool bEnable);

	// Toggle race detection until a call to the same function that toggles it back off.
	// If multiple threads activate race detection at the same time, it will be turned off when all thread toggle it back to off.
	// The grace period is how much time to wait until race detection is really turned off. Can be used when toggling is expected to happen too often or too fast.
	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES CORE_API void ToggleRaceDetection(bool bEnable, float GracePeriodSeconds = 0.0f);
	// Toggle filter on duplicate races. Does not apply when a callback is set.
	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES CORE_API void ToggleFilterDuplicateRaces(bool bEnable);
	// Activate race detection for certain time after which it will be automatically turned off.
	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES CORE_API void ToggleRaceDetectionUntil(UE::FTimeout Timeout);

	// Toggle detailed logging globally for all threads. Very slow!
	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES CORE_API void ToggleGlobalDetailedLog(bool bEnable);
	// Toggle detailed logging for the current thread.
	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES CORE_API void ToggleThreadDetailedLog(bool bEnable);
	// Toggle detailed logging filter to only log activity for a specific address.
	// NOTE: Set to nullptr to remove filtering.
	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES CORE_API void ToggleFilterDetailedLogOnAddress(void* Ptr);
	// Returns whether the race detector is currently active or not.
	INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES CORE_API bool IsActive();
	
	class FRaceDetectorScope
	{
		bool bConditionalActivation;
		float GracePeriodSeconds;
	public:
		FRaceDetectorScope(bool bInConditionalActivation = true, float InGracePeriodSeconds = 0.0f)
			: bConditionalActivation(bInConditionalActivation)
			, GracePeriodSeconds(InGracePeriodSeconds)
		{
			if (bConditionalActivation)
			{
				ToggleRaceDetection(true);
			}
		}

		~FRaceDetectorScope()
		{
			if (bConditionalActivation)
			{
				ToggleRaceDetection(false, GracePeriodSeconds);
			}
		}
	};


} // UE::Sanitizer::RaceDetector

#endif