// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "CoreTypes.h"
#include "HAL/PlatformProperties.h"
#include "Internationalization/TextNamespaceFwd.h"
#include "Math/MathFwd.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Build.h"
#include "Misc/CompressionFlags.h"
#include "Misc/EngineVersionBase.h"
#include "Misc/VarArgs.h"
#include "Serialization/ArchiveCookData.h"
#include "Serialization/ArchiveSavePackageData.h"
#include "Templates/EnableIf.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Templates/IsEnumClass.h"
#include "Templates/IsSigned.h"
#include "Templates/IsValidVariadicFunctionArg.h"
#include "Templates/UnrealTemplate.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "UObject/ObjectVersion.h"

class FArchive;
class FBulkData;
class FCustomVersionContainer;
class FLinker;
class FName;
class FProperty;
class FString;
class FText;
class ITargetPlatform;
class UObject;
namespace Verse { struct VCell; }
namespace UE::Serialization{ class FEditorBulkData; }
struct FArchiveSerializedPropertyChain;
struct FBulkDataSerializationParams;
struct FCompressedChunk;
template <typename TEnum> class TEnumAsByte;
template <typename FuncType> class TFunction;
template <typename T> struct TObjectPtr;

typedef TFunction<bool (double RemainingTime)> FExternalReadCallback;
class FField;
struct FUObjectMigrationContext;
struct FUObjectSerializeContext;

enum class EFileRegionType : uint8;
struct FArchiveCookContext;

#if !defined(USE_EVENT_DRIVEN_ASYNC_LOAD_AT_BOOT_TIME)
#define USE_EVENT_DRIVEN_ASYNC_LOAD_AT_BOOT_TIME UE_DEPRECATED_MACRO(5.8, "USE_EVENT_DRIVEN_ASYNC_LOAD_AT_BOOT_TIME has been deprecated because it is no longer necessary.") 1
#endif 

#if !defined(EVENT_DRIVEN_ASYNC_LOAD_ACTIVE_AT_RUNTIME)
#define EVENT_DRIVEN_ASYNC_LOAD_ACTIVE_AT_RUNTIME UE_DEPRECATED_MACRO(5.8, "EVENT_DRIVEN_ASYNC_LOAD_ACTIVE_AT_RUNTIME has been deprecated because it is no longer necessary.") 1
#endif 

#define DEVIRTUALIZE_FLinkerLoad_Serialize (!WITH_EDITORONLY_DATA)

// Helper macro to make serializing a bitpacked boolean in an archive easier. 
// NOTE: The condition is there to avoid overwriting a value that is the same, especially important to make saving an immutable operation and avoid dirtying cachelines for nothing.
//       This will also make TSAN happy when multiple threads are hitting the same bitfield during concurrent saves.
#define FArchive_Serialize_BitfieldBool(ARCHIVE, BITFIELD_BOOL) { bool TEMP_BITFIELD_BOOL = BITFIELD_BOOL; ARCHIVE << TEMP_BITFIELD_BOOL; if (BITFIELD_BOOL != TEMP_BITFIELD_BOOL) { BITFIELD_BOOL = TEMP_BITFIELD_BOOL; } }

#if UE_INTERNAL_LINKER_BOUNDS
// Named pair to identify bounds, used by some FArchive implementations
struct FArchiveBounds
{
	int64_t BoundsBasis = 0;
	int64_t BoundsSize = 0;
	bool operator==(const FArchiveBounds& Other) const = default;
};

// Helper for scoping bounds of a read from buffer aka load operation
// meant to be created via FArchive::SetBounds. This is only used in
// high placeholder environments but could be appropriate for other
// unreliable reads as well.
struct FArchiveBoundsScope
{
	CORE_API ~FArchiveBoundsScope();
private:
	friend class FArchive;

	FArchiveBoundsScope(FArchive* InOwner, FArchiveBounds Bounds);
	FArchive* Owner;
	FArchiveBounds OriginalBounds;

	FArchiveBoundsScope& operator=(const FArchiveBoundsScope&) = delete;
	FArchiveBoundsScope(const FArchiveBoundsScope&) = delete;
	FArchiveBoundsScope& operator=(FArchiveBoundsScope&&) = delete;
	FArchiveBoundsScope(FArchiveBoundsScope&&) = delete;
};
#endif // UE_INTERNAL_LINKER_BOUNDS

struct FArchiveState
{
private:
	// Only FArchive is allowed to instantiate this, by inheritance
	friend class FArchive;

	CORE_API FArchiveState();

	/** Copy constructor. */
	CORE_API FArchiveState(const FArchiveState&);

	/**
	 * Copy assignment operator.
	 *
	 * @param ArchiveToCopy The archive to copy from.
	 */
	CORE_API FArchiveState& operator=(const FArchiveState& ArchiveToCopy);

	CORE_API virtual ~FArchiveState() = 0;

protected:
	static CORE_API void LinkProxy(FArchiveState& Inner, FArchiveState& Proxy);
	static CORE_API void UnlinkProxy(FArchiveState& Inner, FArchiveState& Proxy);

public:
	/** 
	 * Returns lowest level archive state, proxy archives will override this.
	 */
	virtual FArchiveState& GetInnermostState()
	{
		return *this;
	}

	/** 
	 * Modifies current archive state, can be used to override flags.
	 */
	CORE_API void SetArchiveState(const FArchiveState& InState);

	/**
	 * Sets ArIsError to true. Also sets error in the proxy archiver if one is wrapping this.
	 */
	CORE_API void SetError();

	/**
	 * Sets ArIsError to false, this does not clear any CriticalErrors
	 */
	CORE_API void ClearError();

	/**
	 * Sets the archiver IsCriticalError and IsError to true. Also sets CriticalError in the proxy archiver if one is wrapping this.
	 */
	CORE_API void SetCriticalError();
	
#if UE_INTERNAL_LINKER_BOUNDS
	/**
	 * Sets ArIsError and ArIsCriticalError to false, useful only if recoverable bounds are available
	 */
	CORE_API void ClearCriticalError();
#endif // UE_INTERNAL_LINKER_BOUNDS

	/**
	 * Called to get the computed size from a size-detecting archive after it has finished serializing.
	 */
	virtual void CountBytes(SIZE_T InNum, SIZE_T InMax) { }

	/**
	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 */
	CORE_API virtual FString GetArchiveName() const;

	/**
	 * If this archive is a FLinkerLoad or FLinkerSave, returns a pointer to the ULinker portion.
	 *
	 * @return The linker, or nullptr if the archive is not a linker.
	 */
	virtual FLinker* GetLinker()
	{
		return nullptr;
	}

	/** 
	 * Returns the current location within the backing data storage, which can possibly be passed to Seek later to restore a read/write location.
	 * If this returns -1, there is no backing data storage and Seek will not function.
	 */
	virtual int64 Tell()
	{
		return INDEX_NONE;
	}

	/** Returns total size of the backing data storage. */
	virtual int64 TotalSize()
	{
		return INDEX_NONE;
	}

	/** Returns true if the current location within the backing data storage is at the end, always returns false if there is no storage. */
	virtual bool AtEnd()
	{
		int64 Pos = Tell();

		return ((Pos != INDEX_NONE) && (Pos >= TotalSize()));
	}

	/** Returns true if data larger than 1 byte should be swapped to deal with endian mismatches. */
	inline bool IsByteSwapping()
	{
	#if PLATFORM_LITTLE_ENDIAN
		bool SwapBytes = ArForceByteSwapping;
	#else
		bool SwapBytes = this->IsPersistent();
	#endif
		return SwapBytes;
	}

	/** Sets a flag indicating that this archive contains native or generated code. */
	void ThisContainsCode()
	{
		ArContainsCode = true;
	}

	/** Sets a flag indicating that this archive contains a ULevel or UWorld object. */
	void ThisContainsMap()
	{
		ArContainsMap = true;
	}

	/** Sets a flag indicating that this archive contains data required to be gathered for localization. */
	CORE_API void ThisRequiresLocalizationGather();

	/**
	 * Called to retrieve the archetype from the event driven loader. 
	 * If this returns null, then call GetArchetype yourself. 
	 */
	virtual UObject* GetArchetypeFromLoader(const UObject* Obj)
	{
		return nullptr;
	}

	/** Returns the global engine serialization version used for this archive. */
	UE_FORCEINLINE_HINT FPackageFileVersion UEVer() const
	{
		return ArUEVer;
	}

	/** Returns the licensee-specific version used for this archive, will be 0 by default. */
	UE_FORCEINLINE_HINT int32 LicenseeUEVer() const
	{
		return ArLicenseeUEVer;
	}

	/** Returns the compiled engine version used for this archive. */
	UE_FORCEINLINE_HINT FEngineVersionBase EngineVer() const
	{
		return ArEngineVer;
	}

	/** Returns the engine-global network protocol version for this archive. */
	CORE_API virtual uint32 EngineNetVer() const;

	/** Returns the game-specific network protocol version for this archive. */
	CORE_API virtual uint32 GameNetVer() const;

	/**
	 * Queries a custom version from the archive.  If the archive is being used to write, the custom version must have already been registered.
	 *
	 * @param Key The guid of the custom version to query.
	 * @return The version number, or -1 if the custom tag isn't stored in the archive.
	 */
	CORE_API int32 CustomVer(const struct FGuid& Key) const;

	/** Returns true if this archive is for loading data. */
	UE_FORCEINLINE_HINT bool IsLoading() const
	{
		return ArIsLoading;
	}

	/** Returns true if this archive is loading from a cooked package. */
 	UE_FORCEINLINE_HINT bool IsLoadingFromCookedPackage() const
	{
		return ArIsLoadingFromCookedPackage;
	}

	/** Returns true if this archive is for saving data, this can also be a pre-save preparation archive. */
	UE_FORCEINLINE_HINT bool IsSaving() const
	{
		return ArIsSaving;
	}

	/** Returns true if this archive is transacting, which is used to keep track of changes to objects for things like the editor undo system. */
	inline bool IsTransacting() const
	{
		if (FPlatformProperties::HasEditorOnlyData())
		{
			return ArIsTransacting;
		}
		else
		{
			return false;
		}
	}

	/** 
	 * Returns true if this archive serializes to a structured text format. 
	 * Text format archives should use high level constructs from FStructuredArchive for delimiting data rather than manually seeking through the file.
	 */
	UE_FORCEINLINE_HINT bool IsTextFormat() const
	{
		return (ArIsTextFormat && WITH_TEXT_ARCHIVE_SUPPORT);
	}

	/** Returns true if this archive wants properties to be serialized in binary form instead of safer but slower tagged form. */
	UE_FORCEINLINE_HINT bool WantBinaryPropertySerialization() const
	{
		return ArWantBinaryPropertySerialization;
	}

	/** 
	 * Returns true if tagged property serialization should be replaced by faster unversioned serialization.
	 * This assumes writer and reader share the same property definitions. 
	 */
	UE_FORCEINLINE_HINT bool UseUnversionedPropertySerialization() const
	{
		return ArUseUnversionedPropertySerialization;
	}

	/** Returns true if this archive wants to always save strings in UTF16 format even if they are ANSI characters. */
	UE_FORCEINLINE_HINT bool IsForcingUnicode() const
	{
		return ArForceUnicode;
	}

	/** 
	 * Returns true if this archive is saving or loading data destined for persistent storage and should skip transient data.
	 * This is also true for some intermediate archives for tasks like duplication that are eventually destined for persistent storage.
	 */
	UE_FORCEINLINE_HINT bool IsPersistent() const
	{
		return ArIsPersistent;
	}

	/**
	 * Set whether or not it is allowed to skip serialization on assets still being compiled to avoid waiting unless strictly necessary.
	 * 
	 * @param Enabled	Set to true to enable skip serialization on compiling assets.
	 */
	void SetShouldSkipCompilingAssets(bool Enabled)
	{
		ArShouldSkipCompilingAssets = Enabled;
	}

	/**
	 * Returns true if it is allowed to skip serialization on assets still being compiled to avoid waiting unless strictly necessary. 
	 */
	UE_FORCEINLINE_HINT bool ShouldSkipCompilingAssets() const
	{
		return ArShouldSkipCompilingAssets;
	}

	/** If set true, UsingCurrentVersion will not query the version registry to get the latest version data for the key if the version key is already set in the local container.
	 * Querying the version registry is not cheap, and the look up time increases as more custom versions are registered, therefore if you don't change the registered custom version data once is set in the archive
	 * Setting this to true should improve serialization performance by skipping the step entirely.
	 * @param bShouldSkip Whether we should skip the checking the version registry for new version data if the version key is already set
	 */
	CORE_API void SetShouldSkipUpdateCustomVersion(bool bShouldSkip);

	/** Returns true if this Archive does not update the custom version data for a given key if has already been set previously in the local custom versions container*/
	UE_FORCEINLINE_HINT bool ShouldSkipUpdateCustomVersion() const
	{
		return ArShouldSkipUpdateCustomVersion;
	}

	/** 
	 * If set true, this archive is migrating remote objects
	 * @param bMigrating Whether this archive is migrating remote objects
	 */
	CORE_API void SetMigratingRemoteObjects(bool bMigrating);

	/** When migrating objects, we may need some context as to the intent of the migration during serialization */
	virtual const FUObjectMigrationContext* GetMigrationContext() const
	{
		return nullptr;
	}

	/** Returns true if this Archive is migrating remote object */
#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE_FORCEINLINE_HINT bool IsMigratingRemoteObjects() const
	{
		return ArIsMigratingRemoteObjects;
	}
#else
	UE_FORCEINLINE_HINT constexpr bool IsMigratingRemoteObjects() const
	{
		return false;
	}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

	/** Returns true if this archive contains errors, which means that further serialization is generally not safe. */
	UE_FORCEINLINE_HINT bool IsError() const
	{
		return ArIsError;
	}

	UE_FORCEINLINE_HINT bool GetError() const
	{
		return ArIsError;
	}

	/** Returns true if this archive contains critical errors that cannot be recovered from. */
	UE_FORCEINLINE_HINT bool IsCriticalError() const
	{
		return ArIsCriticalError;
	}

	/** Returns true if this archive contains native or generated code. */
	UE_FORCEINLINE_HINT bool ContainsCode() const
	{
		return ArContainsCode;
	}

	/** Returns true if this archive contains a ULevel or UWorld object. */
	UE_FORCEINLINE_HINT bool ContainsMap() const
	{
		return ArContainsMap;
	}

	/** Returns true if this archive contains data required to be gathered for localization. */
	inline bool RequiresLocalizationGather() const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ArRequiresLocalizationGather;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Returns true if this archive should always swap bytes, ignoring endian rules. */
	UE_FORCEINLINE_HINT bool ForceByteSwapping() const
	{
		return ArForceByteSwapping;
	}

	/** Returns true if this archive is currently serializing class/struct default values. */
	UE_FORCEINLINE_HINT bool IsSerializingDefaults() const
	{
		return (ArSerializingDefaults > 0) ? true : false;
	}

	/** Returns true if this archive should ignore archetype references for structs and classes. */
	UE_FORCEINLINE_HINT bool IsIgnoringArchetypeRef() const
	{
		return ArIgnoreArchetypeRef;
	}

	/** Returns true if this archive should handle delta serialization for properties. */
	UE_FORCEINLINE_HINT bool DoDelta() const
	{
		return !ArNoDelta;
	}

	/** Returns true if this archive should perform delta serialization within properties (e.g. TMaps and TSets). */
	UE_FORCEINLINE_HINT bool DoIntraPropertyDelta() const
	{
		return !ArNoIntraPropertyDelta;
	}

	/** Returns true if this archive should ignore the Outer reference in UObject. */
	UE_FORCEINLINE_HINT bool IsIgnoringOuterRef() const
	{
		return ArIgnoreOuterRef;
	}

	/** Returns true if this archive should ignore the ClassGeneratedBy reference in UClass. */
	UE_FORCEINLINE_HINT bool IsIgnoringClassGeneratedByRef() const
	{
		return ArIgnoreClassGeneratedByRef;
	}

	/** Returns true if this archive should ignore the Class reference in UObject. */
	UE_FORCEINLINE_HINT bool IsIgnoringClassRef() const
	{
		return ArIgnoreClassRef;
	}

	/** Returns true if this archive sould allow lazy loading of bulk / secondary data. */
	UE_FORCEINLINE_HINT bool IsAllowingLazyLoading() const
	{
		return ArAllowLazyLoading;
	}

	/** 
	 * Returns true if this archive is only looking for UObject references.
	 * This can be false for reference collectors looking for more general references.
	 */
	UE_FORCEINLINE_HINT bool IsObjectReferenceCollector() const
	{
		return ArIsObjectReferenceCollector;
	}

	/** Returns true if this archive should modify/search weak object references as well as strong ones. */
	UE_FORCEINLINE_HINT bool IsModifyingWeakAndStrongReferences() const
	{
		return ArIsModifyingWeakAndStrongReferences;
	}

	/** Returns true if this archive is counting memory, normally CountBytes is called to get the size. */
	UE_FORCEINLINE_HINT bool IsCountingMemory() const
	{
		return ArIsCountingMemory;
	}

	/** Returns this archive's property serialization modifier flags. */
	UE_FORCEINLINE_HINT uint32 GetPortFlags() const
	{
		return ArPortFlags;
	}

	/** Checks to see if any of the passed in property serialization modifier flags are set. */
	UE_FORCEINLINE_HINT bool HasAnyPortFlags(uint32 Flags) const
	{
		return ((ArPortFlags & Flags) != 0);
	}

	/** Checks to see if all of the passed in property serialization modifier flags are set. */
	UE_FORCEINLINE_HINT bool HasAllPortFlags(uint32 Flags) const
	{
		return ((ArPortFlags & Flags) == Flags);
	}

	/** Returns the editor-only debug serialization flags. */
	inline uint32 GetDebugSerializationFlags() const
	{
#if WITH_EDITOR
		return ArDebugSerializationFlags;
#else
		return 0;
#endif
	}

	/** Returns true if this archive should ignore bulk data. */
	UE_FORCEINLINE_HINT bool ShouldSkipBulkData() const
	{
		return ArShouldSkipBulkData;
	}

	/** Returns the maximum size of data that this archive is allowed to serialize. */
	UE_FORCEINLINE_HINT int64 GetMaxSerializeSize() const
	{
		return ArMaxSerializeSize;
	}

	/**
	 * Gets the custom version numbers for this archive.
	 * These are used to check for system or game-specific version numbers.
	 *
	 * @return The container of custom versions in the archive.
	 */
	CORE_API virtual const FCustomVersionContainer& GetCustomVersions() const;

	/**
	 * Sets the custom version numbers for this archive.
	 *
	 * @param CustomVersionContainer - The container of custom versions to copy into the archive.
	 */
	CORE_API virtual void SetCustomVersions(const FCustomVersionContainer& CustomVersionContainer);

	/** Resets the custom version numbers for this archive. */
	CORE_API virtual void ResetCustomVersions();

	/**
	 * Sets a specific custom version
	 *
	 * @param Key - The guid of the custom version to query.
	 * @param Version - The version number to set key to
	 * @param FriendlyName - Friendly name corresponding to the key
	 */
	CORE_API void SetCustomVersion(const struct FGuid& Key, int32 Version, FName FriendlyName);

	/**
	 * Toggle byte order swapping. This is needed in rare cases when we already know that the data
	 * swapping has already occurred or if we know that it will be handled later.
	 *
	 * @param Enabled	set to true to enable byte order swapping
	 */
	void SetByteSwapping(bool Enabled)
	{
		ArForceByteSwapping = Enabled;
	}

	/**
	 * Sets the archive's property serialization modifier flags
	 *
	 * @param	InPortFlags		the new flags to use for property serialization
	 */
	void SetPortFlags(uint32 InPortFlags)
	{
		ArPortFlags = InPortFlags;
	}

	/**
	 * Sets the archives custom serialization modifier flags (nothing to do with PortFlags or Custom versions)
	 *
	 * @param InCustomFlags the new flags to use for custom serialization
	 */
	virtual void SetDebugSerializationFlags(uint32 InCustomFlags)
	{
#if WITH_EDITOR
		ArDebugSerializationFlags = InCustomFlags;
#endif
	}

	/**
	 * Indicates whether this archive is filtering editor-only on save or contains data that had editor-only content stripped.
	 *
	 * @return true if the archive filters editor-only content, false otherwise.
	 */
	bool IsFilterEditorOnly() const
	{
		return ArIsFilterEditorOnly;
	}

	/**
	 * Sets a flag indicating that this archive needs to filter editor-only content.
	 *
	 * @param InFilterEditorOnly Whether to filter editor-only content.
	 */
	virtual void SetFilterEditorOnly(bool InFilterEditorOnly)
	{
		ArIsFilterEditorOnly = InFilterEditorOnly;
	}

	/**
	 * Indicates when the archive is serializing an Optional object. Will only return true
	 * when Ar.IsSaving() is also true
	 * @see EClassFlags::CLASS_Optional
	 *
	 * @return true if currently saving optional data
	 */
	bool IsSavingOptionalObject () const
	{
		return ArIsSavingOptionalObject;
	}

	/**
	 * Set whether or not this archive is current serializing an Optional object
	 * @see EClassFlags::CLASS_Optional
	 *
	 * @param InFilterEditorOnly Whether to filter editor-only content.
	 */
	virtual void SetSavingOptionalObject(bool InSavingOptionalObject)
	{
		check(IsSaving());
		ArIsSavingOptionalObject = InSavingOptionalObject;
	}

	/**
	 * Indicates whether this archive is saving or loading game state
	 *
	 * @note This is intended for game-specific archives and is not true for any of the build in save methods
	 * @return true if the archive is dealing with save games, false otherwise.
	 */
	bool IsSaveGame() const
	{
		return ArIsSaveGame;
	}

	/**
	 * Whether or not this archive is serializing data being sent/received by the netcode
	 */
	UE_FORCEINLINE_HINT bool IsNetArchive() const
	{
		return ArIsNetArchive;
	}

	/**
	 * Checks whether the archive is used for cooking.
	 *
	 * @return true if the archive is used for cooking, false otherwise.
	 */
	UE_FORCEINLINE_HINT bool IsCooking() const
	{
		return SavePackageData && SavePackageData->CookContext;
	}

	/**
	 * Marks that this archive is one of the archives used by SavePackage, and provides access
	 * to data and functions used only during savepackage, such as the cook data bundle for information about
	 * the cook being conducted (if applicable) and writing build dependencies for assetregistry.
	 * Must be set only after setting the loading/saving/transacting flags.
	 */
	void SetSavePackageData(FArchiveSavePackageData* InSavePackageData)
	{
		checkf(!(InSavePackageData == nullptr && SavePackageData), TEXT("Can't turn off cooking once you turn it on!"));

		if (InSavePackageData)
		{
			check(!IsLoading() && !IsTransacting() && IsSaving());
			SavePackageData = InSavePackageData;
		}

	}

	FArchiveSavePackageData* GetSavePackageData()
	{
		return SavePackageData;
	}

	UE_FORCEINLINE_HINT FArchiveCookContext* GetCookContext()
	{
		return SavePackageData ? SavePackageData->CookContext : nullptr;
	}

	/**
	 * Returns the cooking target platform.
	 * Returns null if not being serialized from Savepackage, or if it is an editor save rather than a cook save.
	 *
	 * @return Target platform.
	 */
	UE_FORCEINLINE_HINT const ITargetPlatform* CookingTarget() const
	{
		return SavePackageData ? SavePackageData->TargetPlatform : nullptr;
	}

	/**
	 * Return the API object used to record extra data for SavePackage calls (e.g. cook build dependencies).
	 * Returns null if not being serialized from SavePackage.
	 */
	FObjectSavePackageSerializeContext* GetSavePackageSerializeContext()
	{
		return SavePackageData ? &SavePackageData->SavePackageContext : nullptr;
	}

	/**
	 * Checks whether the archive is used to resolve out-of-date enum indexes
	 * If function returns true, the archive should be called only for objects containing user defined enum
	 *
	 * @return true if the archive is used to resolve out-of-date enum indexes
	 */
	virtual bool UseToResolveEnumerators() const
	{
		return false;
	}

	/**
	 * Checks whether the archive wants to skip the property independent of the other flags
	 */
	virtual bool ShouldSkipProperty(const FProperty* InProperty) const
	{
		return false;
	}

	/**
	 * Overrides the property that is currently being serialized
	 * @note: You likely want to call PushSerializedProperty/PopSerializedProperty instead
	 *
	 * @param InProperty Pointer to the property that is currently being serialized
	 */
	virtual void SetSerializedProperty(FProperty* InProperty)
	{
		SerializedProperty = InProperty;
	}

	/**
	 * Gets the property that is currently being serialized
	 *
	 * @return Pointer to the property that is currently being serialized
	 */
	UE_FORCEINLINE_HINT class FProperty* GetSerializedProperty() const
	{
		return SerializedProperty;
	}

	/**
	 * Gets the chain of properties that are currently being serialized
	 * @note This populates the array in stack order, so the 0th entry in the array is the top of the stack of properties
	 */
	CORE_API void GetSerializedPropertyChain(TArray<class FProperty*>& OutProperties) const;

	/**
	 * Get the raw serialized property chain for this archive
	 * @note Accessing this directly can avoid an array allocation depending on your use-case
	 */
	UE_FORCEINLINE_HINT const FArchiveSerializedPropertyChain* GetSerializedPropertyChain() const
	{
		return SerializedPropertyChain;
	}

	/**
	 * Set the raw serialized property chain for this archive, optionally overriding the serialized property too (or null to use the head of the property chain)
	 */
	CORE_API virtual void SetSerializedPropertyChain(const FArchiveSerializedPropertyChain* InSerializedPropertyChain, class FProperty* InSerializedPropertyOverride = nullptr);

#if WITH_EDITORONLY_DATA
	/** Returns true if the stack of currently serialized properties contains an editor-only property */
	CORE_API virtual bool IsEditorOnlyPropertyOnTheStack() const;
#endif

#if USE_STABLE_LOCALIZATION_KEYS
	/**
	 * Set the localization namespace that this archive should use when serializing text properties.
	 * This is typically the namespace used by the package being serialized (if serializing a package, or an object within a package).
	 */
	CORE_API virtual void SetLocalizationNamespace(const FString& InLocalizationNamespace);

	/**
	 * Get the localization namespace that this archive should use when serializing text properties.
	 * This is typically the namespace used by the package being serialized (if serializing a package, or an object within a package).
	 */
	CORE_API virtual FString GetLocalizationNamespace() const;
#endif // USE_STABLE_LOCALIZATION_KEYS

	/** Resets all of the base archive members. */
	CORE_API virtual void Reset();

public:
	/* These are used for fastpath inline serializers  */
	struct FFastPathLoadBuffer
	{
		const uint8* StartFastPathLoadBuffer;
		const uint8* EndFastPathLoadBuffer;
		const uint8* OriginalFastPathLoadBuffer;
		UE_FORCEINLINE_HINT FFastPathLoadBuffer()
		{
			Reset();
		}
		inline void Reset()
		{
			StartFastPathLoadBuffer = nullptr;
			EndFastPathLoadBuffer = nullptr;
			OriginalFastPathLoadBuffer = nullptr;
		}
	};
#if DEVIRTUALIZE_FLinkerLoad_Serialize
	//@todoio FArchive is really a horrible class and the way it is proxied by FLinkerLoad is double terrible. It makes the fast path really hacky and slower than it would need to be.
	FFastPathLoadBuffer* ActiveFPLB;
	FFastPathLoadBuffer InlineFPLB;
#endif

// These will be private in FArchive
protected:
	/** Copies all of the members except CustomVersionContainer */
	CORE_API void CopyTrivialFArchiveStatusMembers(const FArchiveState& ArchiveStatusToCopy);

	/** Whether this archive is for loading data. */
	uint8 ArIsLoading : 1;

	/** Whether this archive is loading from a cooked package. */
	uint8 ArIsLoadingFromCookedPackage : 1;

	/** Whether this archive is for saving data. */
	uint8 ArIsSaving : 1;

	/** Whether archive is transacting, which is used to keep track of changes to objects for things like the editor undo system. */
	uint8 ArIsTransacting : 1;

	/** Whether this archive serializes to a text format. Text format archives should use high level constructs from FStructuredArchive for delimiting data rather than manually seeking through the file. */
	uint8 ArIsTextFormat : 1;

	/** Whether this archive wants properties to be serialized in binary form instead of tagged. */
	uint8 ArWantBinaryPropertySerialization : 1;

	/** Whether tagged property serialization is replaced by faster unversioned serialization. This assumes writer and reader share the same property definitions. */
	uint8 ArUseUnversionedPropertySerialization : 1;

	/** Whether this archive wants to always save strings in UTF16 format even if they are ANSI characters */
	uint8 ArForceUnicode : 1;

	/** Whether this archive saves to persistent storage. This is also true for some intermediate archives like DuplicateObject that are expected to go to persistent storage but may be discarded */
	uint8 ArIsPersistent : 1;

private:
	/** Whether this archive contains errors, which means that further serialization is generally not safe */
	uint8 ArIsError : 1;

	/** Whether this archive contains critical errors that cannot be recovered from */
	uint8 ArIsCriticalError : 1;

	/** Whether or not it is allowed to skip serialization on assets still being compiled to avoid waiting unless strictly necessary. */
	uint8 ArShouldSkipCompilingAssets : 1;

	/** If true, UsingCurrentVersion will not query the version registry to get the latest version data for the key if the version key is already set in the local container.
	 * Querying the version registry is not cheap, and the look up time increases as more custom versions are registered, therefore if you don't change the registered custom version data once is set in the archive
	 * Setting this to true should improve serialization performance by skipping the step entirely */
	uint8 ArShouldSkipUpdateCustomVersion : 1;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	/** If true this archive is migrating remote object data. */
	uint8 ArIsMigratingRemoteObjects : 1;
#endif

public:
	/** Quickly tell if an archive contains script code. */
	uint8 ArContainsCode : 1;

	/** Used to determine whether FArchive contains a level or world. */
	uint8 ArContainsMap : 1;

	/** Used to determine whether FArchive contains data required to be gathered for localization. */
	UE_DEPRECATED(5.6, "Public access to ArRequiresLocalizationGather has been deprecated - please use Ar.RequiresLocalizationGather() and Ar.ThisRequiresLocalizationGather() for getting and setting respectively.")
	uint8 ArRequiresLocalizationGather : 1;

	/** Whether we should forcefully swap bytes. */
	uint8 ArForceByteSwapping : 1;

	/** If true, we will not serialize archetype references for structs and classes. */
	uint8 ArIgnoreArchetypeRef : 1;

	/** If true, do not perform delta serialization of properties. */
	uint8 ArNoDelta : 1;

	/** If true, do not perform delta serialization within properties (e.g. TMaps and TSets). */
	uint8 ArNoIntraPropertyDelta : 1;

	/** If true, we will not serialize the Outer reference in UObject. */
	uint8 ArIgnoreOuterRef : 1;

	/** If true, we will not serialize ClassGeneratedBy reference in UClass. */
	uint8 ArIgnoreClassGeneratedByRef : 1;

	/** If true, UObject::Serialize will skip serialization of the Class property. */
	uint8 ArIgnoreClassRef : 1;

	/** Whether to allow lazy loading of bulk/secondary data. */
	uint8 ArAllowLazyLoading : 1;

	/** Whether this archive only cares about serializing object references. */
	uint8 ArIsObjectReferenceCollector : 1;

	/** Whether a reference collector is modifying the references and wants both weak and strong ones */
	uint8 ArIsModifyingWeakAndStrongReferences : 1;

	/** Whether this archive is counting memory. */
	uint8 ArIsCountingMemory : 1;

	/** Whether bulk data serialization should be skipped or not. */
	uint8 ArShouldSkipBulkData : 1;

	/** Whether editor only properties are being filtered from the archive (or has been filtered). */
	uint8 ArIsFilterEditorOnly : 1;

	/** Whether we are currently serializing optional data */
	uint8 ArIsSavingOptionalObject : 1;

	/** Whether this archive is saving/loading game state */
	uint8 ArIsSaveGame : 1;

	/** Whether or not this archive is sending/receiving network data */
	uint8 ArIsNetArchive : 1;

	/** Set TRUE to use the custom property list attribute for serialization. */
	uint8 ArUseCustomPropertyList : 1;

	/** Set to false if OverriddenProperties should be cleared at the beginning of SerializeVersionedTaggedProperties. */
	uint8 ArMergeOverrides : 1;

	/** Set TRUE to stop arrays from being cleared  before they're loaded. Instead, array data will be merged */
	uint8 ArPreserveArrayElements : 1;

	/** Whether we are currently serializing defaults. > 0 means yes, <= 0 means no. */
	int32 ArSerializingDefaults;

	/** Modifier flags that be used when serializing UProperties */
	uint32 ArPortFlags;

	/** Max size of data that this archive is allowed to serialize. */
	int64 ArMaxSerializeSize;

	/**
	 * Sets whether this archive is for loading data.
	 *
	 * @param bInIsLoading  true if this archive is for loading, false otherwise.
	 */
	CORE_API virtual void SetIsLoading(bool bInIsLoading);

	/** 
	 * Sets whether the archive is loading from a cooked package.
	 * @param bInIsLoadingFromCookedPackage  true if this archive is loading from a cooked package, false otherwise
	 */
	CORE_API virtual void SetIsLoadingFromCookedPackage(bool bInIsLoadingFromCookedPackage);

	/**
	 * Sets whether this archive is for saving data.
	 *
	 * @param bInIsSaving  true if this archive is for saving, false otherwise.
	 */
	CORE_API virtual void SetIsSaving(bool bInIsSaving);

	/**
	 * Sets whether this archive is for transacting.
	 *
	 * @param bInIsTransacting  true if this archive is for transacting, false otherwise.
	 */
	CORE_API virtual void SetIsTransacting(bool bInIsTransacting);

	/**
	 * Sets whether this archive is in text format.
	 *
	 * @param bInIsTextFormat  true if this archive is in text format, false otherwise.
	 */
	CORE_API virtual void SetIsTextFormat(bool bInIsTextFormat);

	/**
	 * Sets whether this archive wants binary property serialization.
	 *
	 * @param bInWantBinaryPropertySerialization  true if this archive wants binary serialization, false otherwise.
	 */
	CORE_API virtual void SetWantBinaryPropertySerialization(bool bInWantBinaryPropertySerialization);

	/** Sets whether tagged property serialization should be replaced by faster unversioned serialization. This assumes writer and reader share the same property definitions. */
	CORE_API virtual void SetUseUnversionedPropertySerialization(bool bInUseUnversioned);

	/**
	 * Sets whether this archive wants to force saving as Unicode.
	 * This is needed when we need to make sure ANSI strings are saved as Unicode.
	 *
	 * @param bInForceUnicode  true if this archive wants to force saving as Unicode, false otherwise.
	 */
	CORE_API virtual void SetForceUnicode(bool bInForceUnicode);

	/**
	 * Sets whether this archive is to persistent storage.
	 *
	 * @param bInIsPersistent  true if this archive is to persistent storage, false otherwise.
	 */
	CORE_API virtual void SetIsPersistent(bool bInIsPersistent);

	/**
	 * Sets the archive version number. Used by the code that makes sure that FLinkerLoad's 
	 * internal archive versions match the file reader it creates.
	 *
	 * @param UEVer	new version number
	 */
	CORE_API virtual void SetUEVer(FPackageFileVersion InVer);

	/**
	 * Sets the archive licensee version number. Used by the code that makes sure that FLinkerLoad's 
	 * internal archive versions match the file reader it creates.
	 *
	 * @param Ver	new version number
	 */
	CORE_API virtual void SetLicenseeUEVer(int32 InVer);

	/**
	 * Sets the archive engine version. Used by the code that makes sure that FLinkerLoad's
	 * internal archive versions match the file reader it creates.
	 *
	 * @param InVer	new version number
	 */
	CORE_API virtual void SetEngineVer(const FEngineVersionBase& InVer);

	/**
	 * Sets the archive engine network version.
	 */
	CORE_API virtual void SetEngineNetVer(const uint32 InEngineNetVer);

	/**
	 * Sets the archive game network version.
	 */
	CORE_API virtual void SetGameNetVer(const uint32 InGameNetVer);

// These will be private in FArchive
protected:
	/** Holds the archive version. */
	FPackageFileVersion ArUEVer;

	/** Holds the archive version for licensees. */
	int32 ArLicenseeUEVer;

	/** Holds the engine version. */
	FEngineVersionBase ArEngineVer;

	/**
	* All the custom versions stored in the archive.
	* Stored as a pointer to a heap-allocated object because of a 3-way dependency between TArray, FCustomVersionContainer and FArchive, which is too much work to change right now.
	* Keeping it as a heap-allocated object also helps with performance in some cases as we don't need to construct it for archives that don't care about custom versions.
	*/
	mutable FCustomVersionContainer* CustomVersionContainer = nullptr;

public:
	/** Custom property list attribute. If the flag below is set, only these properties will be iterated during serialization. If NULL, then no properties will be iterated. */
	const struct FCustomPropertyListNode* ArCustomPropertyList;

#if WITH_EDITOR
	/** Custom serialization modifier flags can be used for anything */
	uint32 ArDebugSerializationFlags;
#endif

// These will be private in FArchive
protected:
	/**
	 * Provider of data and API specific to UPackage::Save. Required by archives used from UPackage::Save, and required for cook saves of packages.
	 * nullptr means the archive is not one from UPackage::Save and is not being called from a cook save.
	 */
	FArchiveSavePackageData* SavePackageData = nullptr;
	
	/** Holds the pointer to the property that is currently being serialized */
	FProperty* SerializedProperty;

	/** Holds the chain of properties that are currently being serialized */
	FArchiveSerializedPropertyChain* SerializedPropertyChain;

#if USE_STABLE_LOCALIZATION_KEYS
	/**
	 * The localization namespace that this archive should use when serializing text properties.
	 * This is typically the namespace used by the package being serialized (if serializing a package, or an object within a package).
	 * Stored as a pointer to a heap-allocated string because of a dependency between TArray (thus FString) and FArchive; null should be treated as an empty string.
	 */
	FString* LocalizationNamespacePtr;

	/** See GetLocalizationNamespace */
	CORE_API FString GetBaseLocalizationNamespace() const;

	/** See SetLocalizationNamespace */
	CORE_API void SetBaseLocalizationNamespace(const FString& InLocalizationNamespace);
#endif // USE_STABLE_LOCALIZATION_KEYS

	/**
	 * Indicates if the custom versions container is in a 'reset' state.  This will be used to defer the choice about how to
	 * populate the container until it is needed, where the read/write state will be known.
	 */
	mutable bool bCustomVersionsAreReset;

private:
	/** Linked list to all proxies */
	FArchiveState* NextProxy = nullptr;

	template<typename T> void ForEachState(T Func);
};

/**
 * TCheckedObjPtr
 *
 * Wrapper for UObject pointers, which checks that the base class is accurate, upon serializing (to prevent illegal casting)
 */
template<class T> class TCheckedObjPtr
{
	friend class FArchive;

public:
	TCheckedObjPtr()
		: Object(nullptr)
		, bError(false)
	{
	}

	TCheckedObjPtr(T* InObject)
		: Object(InObject)
		, bError(false)
	{
	}

	/**
	 * Assigns a value to the object pointer
	 *
	 * @param InObject	The value to assign to the pointer
	 */
	inline TCheckedObjPtr& operator = (T* InObject)
	{
		Object = InObject;

		return *this;
	}

	/**
	 * Returns the object pointer, for accessing members of the object
	 *
	 * @return	Returns the object pointer
	 */
	UE_FORCEINLINE_HINT T* operator -> () const
	{
		return Object;
	}

	/**
	 * Retrieves a writable/serializable reference to the pointer
	 *
	 * @return	Returns a reference to the pointer
	 */
	UE_FORCEINLINE_HINT T*& Get()
	{
		return Object;
	}

	/**
	 * Whether or not the pointer is valid/non-null
	 *
	 * @return	Whether or not the pointer is valid
	 */
	UE_FORCEINLINE_HINT bool IsValid() const
	{
		return Object != nullptr;
	}

	/**
	 * Whether or not there was an error during the previous serialization.
	 * This occurs if an object was successfully serialized, but with the wrong base class
	 * (which net serialization may have to recover from, if there was supposed to be data serialized along with the object)
	 *
	 * @return	Whether or not there was an error
	 */
	UE_FORCEINLINE_HINT bool IsError() const
	{
		return bError;
	}

private:
	/** The object pointer */
	T* Object;

	/** Whether or not there was an error upon serializing */
	bool bError;
};


/**
 * Base class for archives that can be used for loading, saving, and garbage
 * collecting in a byte order neutral way.
 */
class FArchive : private FArchiveState
{
public:
	FArchive() = default;
	FArchive(const FArchive&) = default;
	FArchive& operator=(const FArchive& ArchiveToCopy) = default;
	CORE_API virtual ~FArchive();

protected:
	using FArchiveState::LinkProxy;
	using FArchiveState::UnlinkProxy;
	using FArchiveState::FFastPathLoadBuffer;

public:

	/**
	 * Serializes an FName value from or into this archive.
	 *
	 * This operator can be implemented by sub-classes that wish to serialize FName instances.
	 *
	 * @param Value The value to serialize.
	 * @return This instance.
	 */
	virtual FArchive& operator<<(FName& Value)
	{
		return *this;
	}

	/**
	 * Serializes an FText value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	CORE_API virtual FArchive& operator<<(FText& Value);

	/**
	 * Serializes an UObject value from or into this archive.
	 *
	 * This operator can be implemented by sub-classes that wish to serialize UObject instances.
	 *
	 * @param Value The value to serialize.
	 * @return This instance.
	 */
	virtual FArchive& operator<<(UObject*& Value)
	{
		return *this;
	}

	/**
	 * Serializes a Field value from or into this archive.
	 *
	 * This operator can be implemented by sub-classes that wish to serialize UObject instances.
	 *
	 * @param Value The value to serialize.
	 * @return This instance.
	 */
	virtual FArchive& operator<<(FField*& Value)
	{
		return *this;
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	/**
	 * Serializes a VCell reference from or into this archive.
	 *
	 * This operator can be implemented by sub-classes that wish to serialize VCell references.
	 *
	 * @param Value The value to serialize.
	 * @return This instance.
	 */
	virtual FArchive& operator<<(Verse::VCell*& Value)
	{
		return *this;
	}
#endif

	/**
	 * Serializes a UObject wrapped in a TCheckedObjPtr container, using the above operator,
	 * and verifies the serialized object is derived from the correct base class, to prevent illegal casting.
	 *
	 * @param Value The value to serialize.
	 * @return This instance.
	 */
	template<class T> inline FArchive& operator<<(TCheckedObjPtr<T>& Value)
	{
		Value.bError = false;

		if (IsSaving())
		{
			UObject* SerializeObj = nullptr;

			if (Value.IsValid())
			{
				if (Value.Get()->IsA(T::StaticClass()))
				{
					SerializeObj = Value.Get();
				}
				else
				{
					Value.bError = true;
				}
			}

			*this << SerializeObj;
		}
		else
		{
			*this << Value.Get();

			if (IsLoading() && Value.IsValid() && !Value.Get()->IsA(T::StaticClass()))
			{
				Value.bError = true;
				Value = nullptr;
			}
		}

		return *this;
	}

	/**
	 * Serializes a lazy object pointer value from or into this archive.
	 *
	 * Most of the time, FLazyObjectPtrs are serialized as UObject*, but some archives need to override this.
	 *
	 * @param Value The value to serialize.
	 * @return This instance.
	 */
	CORE_API virtual FArchive& operator<<(struct FLazyObjectPtr& Value);

	/**
	 * Serializes a wrapped object pointer value from or into this archive.
	 *
	 * Most of the time, FObjectPtrs are serialized as UObject*, but some archives need to override this.
	 *
	 * @param Value The value to serialize.
	 * @return This instance.
	 */
	CORE_API virtual FArchive& operator<<(struct FObjectPtr& Value);
	
	/**
	 * Serializes asset pointer from or into this archive.
	 *
	 * Most of the time, FSoftObjectPtr are serialized as UObject *, but some archives need to override this.
	 *
	 * @param Value The asset pointer to serialize.
	 * @return This instance.
	 */
	CORE_API virtual FArchive& operator<<(struct FSoftObjectPtr& Value);

	/**
	 * Serializes soft object paths from or into this archive.
	 *
	 * @param Value Soft object path to serialize.
	 * @return This instance.
	 */
	CORE_API virtual FArchive& operator<<(struct FSoftObjectPath& Value);

	/**
	* Serializes FWeakObjectPtr value from or into this archive.
	*
	* This operator can be implemented by sub-classes that wish to serialize FWeakObjectPtr instances.
	*
	* @param Value The value to serialize.
	* @return This instance.
	*/
	CORE_API virtual FArchive& operator<<(struct FWeakObjectPtr& Value);

	/** 
	 * Inform the archive that a blueprint would like to force finalization, normally
	 * this is triggered by CDO load, but if there's no CDO we force finalization.
	 */
	virtual void ForceBlueprintFinalization() {}

	/**
	 * Serializes an ANSICHAR value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	inline friend FArchive& operator<<(FArchive& Ar, ANSICHAR& Value)
	{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		if (!Ar.FastPathLoad<sizeof(Value)>(&Value))
#endif
		{
			Ar.Serialize(&Value, 1);
		}
		return Ar;
	}

	/**
	 * Serializes a WIDECHAR value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	inline friend FArchive& operator<<(FArchive& Ar, WIDECHAR& Value)
	{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		if (!Ar.FastPathLoad<sizeof(Value)>(&Value))
#endif
		{
			Ar.ByteOrderSerialize(&Value, sizeof(Value));
		}
		return Ar;
	}

	/**
	 * Serializes a UTF32CHAR value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	inline friend FArchive& operator<<(FArchive& Ar, UTF32CHAR& Value)
	{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		if (!Ar.FastPathLoad<sizeof(Value)>(&Value))
#endif
		{
			Ar.ByteOrderSerialize(&Value, sizeof(Value));
		}
		return Ar;
	}

	/**
	 * Serializes an unsigned 8-bit integer value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	inline friend FArchive& operator<<(FArchive& Ar, uint8& Value)
	{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		if (!Ar.FastPathLoad<sizeof(Value)>(&Value))
#endif
		{
			Ar.Serialize(&Value, 1);
		}
		return Ar;
	}

	/**
	 * Serializes a signed 8-bit integer value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	inline friend FArchive& operator<<(FArchive& Ar, int8& Value)
	{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		if (!Ar.FastPathLoad<sizeof(Value)>(&Value))
#endif
		{
			Ar.Serialize(&Value, 1);
		}
		return Ar;
	}

	/**
	 * Serializes an unsigned 16-bit integer value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	inline friend FArchive& operator<<(FArchive& Ar, uint16& Value)
	{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		if (!Ar.FastPathLoad<sizeof(Value)>(&Value))
#endif
		{
			Ar.ByteOrderSerialize(Value);
		}
		return Ar;
	}

	/**
	 * Serializes a signed 16-bit integer value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	inline friend FArchive& operator<<(FArchive& Ar, int16& Value)
	{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		if (!Ar.FastPathLoad<sizeof(Value)>(&Value))
#endif
		{
			Ar.ByteOrderSerialize(reinterpret_cast<uint16&>(Value));
		}
		return Ar;
	}

	/**
	 * Serializes an unsigned 32-bit integer value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	inline friend FArchive& operator<<(FArchive& Ar, uint32& Value)
	{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		if (!Ar.FastPathLoad<sizeof(Value)>(&Value))
#endif
		{
			Ar.ByteOrderSerialize(Value);
		}
		return Ar;
	}

	/**
	 * Serializes a Boolean value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
#if WITH_EDITOR
protected:
	CORE_API virtual void SerializeBool( bool& D );
public:
	inline friend FArchive& operator<<(FArchive& Ar, bool& D)
	{
		Ar.SerializeBool(D);
		return Ar;
	}
#else
	inline friend FArchive& operator<<( FArchive& Ar, bool& D )
	{
		// Serialize bool as if it were UBOOL (legacy, 32 bit int).
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		const uint8 * RESTRICT Src = Ar.ActiveFPLB->StartFastPathLoadBuffer;
		if (Src + sizeof(uint32) <= Ar.ActiveFPLB->EndFastPathLoadBuffer)
		{
			D = !!FPlatformMemory::ReadUnaligned<uint32>(Src);
			Ar.ActiveFPLB->StartFastPathLoadBuffer += 4;
		}
		else
#endif
		{
			uint32 OldUBoolValue = 0;

			if (!Ar.IsLoading())
			{
				// Only read from the input if we aren't loading its value.
				// This avoids triggering UBSAN by unnecessarily coercing uninitialized memory to a bool.
				OldUBoolValue = D ? 1 : 0;
			}

			Ar.Serialize(&OldUBoolValue, sizeof(OldUBoolValue));

			if (OldUBoolValue > 1)
			{
				Ar.SetError();
			}

			if (Ar.IsLoading())
			{
				// Only write to our input if we are loading to make sure we don't
				// write to 'read-only' memory (e.g. FMemoryWriter) as this can cause TSAN
				// validation races
				D = !!OldUBoolValue;
			}
		}
		return Ar;
	}
#endif

	/**
	 * Serializes a signed 32-bit integer value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	inline friend FArchive& operator<<(FArchive& Ar, int32& Value)
	{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		if (!Ar.FastPathLoad<sizeof(Value)>(&Value))
#endif
		{
			Ar.ByteOrderSerialize(reinterpret_cast<uint32&>(Value));
		}
		return Ar;
	}

#if PLATFORM_COMPILER_DISTINGUISHES_INT_AND_LONG
	/**
	 * Serializes a long integer value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	inline friend FArchive& operator<<(FArchive& Ar, long& Value)
	{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		if (!Ar.FastPathLoad<sizeof(Value)>(&Value))
#endif
		{
			Ar.ByteOrderSerialize(reinterpret_cast<unsigned long&>(Value));
		}
		return Ar;
	}	
#endif

	/**
	 * Serializes a single precision floating point value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	inline friend FArchive& operator<<( FArchive& Ar, float& Value)
	{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		if (!Ar.FastPathLoad<sizeof(Value)>(&Value))
#endif
		{
			static_assert(sizeof(float) == sizeof(uint32), "Expected float to be 4 bytes to swap as uint32");
			Ar.ByteOrderSerialize(reinterpret_cast<uint32&>(Value));
		}
		return Ar;
	}

	/**
	 * Serializes a double precision floating point value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	inline friend FArchive& operator<<(FArchive& Ar, double& Value)
	{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		if (!Ar.FastPathLoad<sizeof(Value)>(&Value))
#endif
		{
			static_assert(sizeof(double) == sizeof(uint64), "Expected double to be 8 bytes to swap as uint64");
			Ar.ByteOrderSerialize(reinterpret_cast<uint64&>(Value));
		}
		return Ar;
	}

	/**
	 * Serializes a unsigned 64-bit integer value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	inline friend FArchive& operator<<(FArchive &Ar, uint64& Value)
	{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		if (!Ar.FastPathLoad<sizeof(Value)>(&Value))
#endif
		{
			Ar.ByteOrderSerialize(Value);
		}
		return Ar;
	}

	/**
	 * Serializes a signed 64-bit integer value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	/*inline*/friend FArchive& operator<<(FArchive& Ar, int64& Value)
	{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		if (!Ar.FastPathLoad<sizeof(Value)>(&Value))
#endif
		{
			Ar.ByteOrderSerialize(reinterpret_cast<uint64&>(Value));
		}
		return Ar;
	}

	/**
	 * Serializes an FString value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	friend CORE_API FArchive& operator<<(FArchive& Ar, FString& Value);

public:
	virtual void Serialize(void* V, int64 Length)
	{
#if defined(__clang_analyzer__)
		// Suppress core.uninitialized.Assign static analysis warning
		// Base class FArchive::Serialize() does not modify the value being serialized, so the analysis
		// assumes any data being serialized to could be uninitialized, but practically speaking any
		// data being serialized will use a derived class that always modifies the value.
		memset(V, 0, Length);
#endif
	}

#define SerializeBitfield(Ar, BitField) \
	{ \
		bool UE_JOIN(TempBitfield, __LINE__) = BitField; \
		Ar.SerializeBits(&UE_JOIN(TempBitfield, __LINE__), 1); \
		BitField = UE_JOIN(TempBitfield, __LINE__); \
	}

	virtual void SerializeBits(void* V, int64 LengthBits)
	{
		Serialize(V, (LengthBits + 7) / 8);

		if (IsLoading() && (LengthBits % 8) != 0)
		{
			((uint8*)V)[LengthBits / 8] &= ((1 << (LengthBits & 7)) - 1);
		}
	}

	virtual void SerializeInt(uint32& Value, uint32 Max)
	{
		ByteOrderSerialize(Value);
	}

	/** Packs int value into bytes of 7 bits with 8th bit for 'more' */
	CORE_API virtual void SerializeIntPacked(uint32& Value);
	CORE_API virtual void SerializeIntPacked64(uint64& Value);

	/** Tells the archive to attempt to preload the specified object so data can be loaded out of it. */
	virtual void Preload(UObject* Object) { }

	/** Returns the low level archive state for this archive. */
	FArchiveState& GetArchiveState()
	{
		return ImplicitConv<FArchiveState&>(*this);
	}

	const FArchiveState& GetArchiveState() const
	{
		return ImplicitConv<const FArchiveState&>(*this);
	}

	using FArchiveState::SetArchiveState;
	using FArchiveState::SetError;
	using FArchiveState::ClearError;
	using FArchiveState::SetCriticalError;
#if UE_INTERNAL_LINKER_BOUNDS
	using FArchiveState::ClearCriticalError;
#endif // UE_INTERNAL_LINKER_BOUNDS
	using FArchiveState::GetInnermostState;
	using FArchiveState::CountBytes;
	using FArchiveState::GetArchiveName;
	using FArchiveState::GetLinker;
	using FArchiveState::Tell;
	using FArchiveState::TotalSize;
	using FArchiveState::AtEnd;

	/** Attempts to set the current offset into backing data storage, this will do nothing if there is no storage. */
	virtual void Seek(int64 InPos) { }

	/**
	 * Attaches/ associates the passed in bulk data object with the linker.
	 *
	 * @param	Owner		UObject owning the bulk data
	 * @param	BulkData	Bulk data object to associate
	 */
	virtual void AttachBulkData(UObject* Owner, FBulkData* BulkData) { }
	virtual void AttachBulkData(UE::Serialization::FEditorBulkData* BulkData) {}

	/**
	 * Detaches the passed in bulk data object from the linker.
	 *
	 * @param	BulkData	Bulk data object to detach
	 * @param	bEnsureBulkDataIsLoaded	Whether to ensure that the bulk data is loaded before detaching
	 */
	virtual void DetachBulkData(FBulkData* BulkData, bool bEnsureBulkDataIsLoaded) { }
	virtual void DetachBulkData(UE::Serialization::FEditorBulkData* BulkData, bool bEnsureBulkDataIsLoaded) {}

	/**
	 * Serialize bulk data.
	 *
	 * @param	BulkData		Bulk data object to serialize
	 * @param	Params			Serialization parameters
	 *
	 * @return true if the bulk data was serialized, false to fallback to default serialization (inline)
	 */
	virtual bool SerializeBulkData(FBulkData& BulkData, const FBulkDataSerializationParams& Params)
	{ 
		return false;
	}

	/**
	* Determine if the given archive is a valid "child" of this archive. In general, this means "is exactly the same" but
	* this function allows a derived archive to support "child" or "internal" archives which are different objects that proxy the
	* original one in some way.
	*
	* @param	BulkData	Bulk data object to detach
	* @param	bEnsureBulkDataIsLoaded	Whether to ensure that the bulk data is loaded before detaching
	*/
	virtual bool IsProxyOf(FArchive* InOther) const
	{
		return InOther == this;
	}

	/**
	 * Hint the archive that the region starting at passed in offset and spanning the passed in size
	 * is going to be read soon and should be precached.
	 *
	 * The function returns whether the precache operation has completed or not which is an important
	 * hint for code knowing that it deals with potential async I/O. The archive is free to either not 
	 * implement this function or only partially precache so it is required that given sufficient time
	 * the function will return true. Archives not based on async I/O should always return true.
	 *
	 * This function will not change the current archive position.
	 *
	 * @param	PrecacheOffset	Offset at which to begin precaching.
	 * @param	PrecacheSize	Number of bytes to precache
	 * @return	false if precache operation is still pending, true otherwise
	 */
	virtual bool Precache(int64 PrecacheOffset, int64 PrecacheSize)
	{
		return true;
	}

	/**
	 * Flushes cache and frees internal data.
	 */
	virtual void FlushCache() { }

	/**
	 * Sets mapping from offsets/ sizes that are going to be used for seeking and serialization to what
	 * is actually stored on disk. If the archive supports dealing with compression in this way it is 
	 * going to return true.
	 *
	 * @param	CompressedChunks	Pointer to array containing information about [un]compressed chunks
	 * @param	CompressionFlags	Flags determining compression format associated with mapping
	 *
	 * @return true if archive supports translating offsets & uncompressing on read, false otherwise
	 */
	virtual bool SetCompressionMap(TArray<FCompressedChunk>* CompressedChunks, ECompressionFlags CompressionFlags)
	{
		return false;
	}

	/** 
	 * Attempts to finish writing any buffered data to disk/permanent storage.
	 */
	virtual void Flush() { }

	/** 
	 * Attempts to close and finalize any handles used for backing data storage, returns true if it succeeded.
	 */
	virtual bool Close()
	{
		return !IsError();
	}

	/**
	 * Serializes and compresses/ uncompresses data. This is a shared helper function for compression support. 
	 *
	 * Do not use SerializeCompressed in new code, prefer SerializeCompressedNew instead.
	 * SerializeCompressedNew can be dropped in to any existing use of SerializeCompressed.
	 *
	 * @param	V		Data pointer to serialize data from/ to
	 * @param	Length	Length of source data if we're saving, unused otherwise
	 * @param   CompressionFormatCannotChange  Compression Format to use for encoding & decoding - cannot be changed without breaking file compatibility
	 * @param	Flags	Flags to control what method to use for [de]compression and optionally control memory vs speed when compressing
	 * @param	bTreatBufferAsFileReader true if V is actually an FArchive, which is used when saving to read data - helps to avoid single huge allocations of source data
	 */
	CORE_API void SerializeCompressed(void* V, int64 Length, FName CompressionFormatCannotChange,
		ECompressionFlags Flags=COMPRESS_NoFlags, bool bTreatBufferAsFileReader=false);
	
	/**
	 * Serializes and compresses/ uncompresses data. This is a shared helper function for compression support. 
	 *
	 * call SerializeCompressedNew instead of SerializeCompressed
	 * Typically you should not serializing data compressed if it will be packaged or stored in the DDC.
	 * Prefer to allow the package/iostore system to do the compression for you instead.
	 *
	 * SerializeCompressedNew can read existing data written by old SerializeCompressed calls.
	 *
	 * @param	V		Data pointer to serialize data from/ to
	 * @param	Length	Length of source data if we're saving, unused otherwise
	 * @param   CompressionFormatToEncode  Compression Format to use for encoding, can be changed freely without breaking compatibility
	 * @param   CompressionFormatToDecodeOldV1Files  Compression Format to decode old data with that didn't write compressor in header, cannot change, usually NAME_Zlib
	 * @param	Flags	Flags to control what method to use for [de]compression and optionally control memory vs speed when compressing
	 * @param	bTreatBufferAsFileReader true if V is actually an FArchive, which is used when saving to read data - helps to avoid single huge allocations of source data
	 * @param	OutPartialReadLength if not null, partial reads are allowed and the size is filled here
	 */
	CORE_API void SerializeCompressedNew(void* V, int64 Length, FName CompressionFormatToEncode, FName CompressionFormatToDecodeOldV1Files,
		ECompressionFlags Flags=COMPRESS_NoFlags, bool bTreatBufferAsFileReader=false,
		int64 * OutPartialReadLength=nullptr);
		
	/**
	 * Serializes and compresses/ uncompresses data with default compressor choices.
	 *
	 * Default compressors are Oodle for new data and Zlib when loading legacy data.
	 *
	 * @param	V		Data pointer to serialize data from/ to
	 * @param	Length	Length of source data if we're saving, unused otherwise
	 */
	CORE_API void SerializeCompressedNew(void* V, int64 Length);


	using FArchiveState::IsByteSwapping;

	/** Used to do byte swapping on small items. This does not happen usually, so we don't want it inline. */
	CORE_API void ByteSwap(void* V, int32 Length);

	/** Serialize data of Length bytes, taking into account byte swapping if needed. */
	inline FArchive& ByteOrderSerialize(void* V, int32 Length)
	{
		if (!IsByteSwapping()) // Most likely case (hot path)
		{
			Serialize(V, Length);
			return *this;
		}
		return SerializeByteOrderSwapped(V, Length); // Slowest and unlikely path (should not be inlined)
	}

	using FArchiveState::ThisContainsCode;
	using FArchiveState::ThisContainsMap;
	using FArchiveState::ThisRequiresLocalizationGather;

	/** Sets a flag indicating that this archive is currently serializing class/struct defaults. */
	void StartSerializingDefaults()
	{
		ArSerializingDefaults++;
	}

	/** Indicate that this archive is no longer serializing class/struct defaults. */
	void StopSerializingDefaults() 
	{
		ArSerializingDefaults--;
	}

	/**
	 * Called when an object begins serializing property data using script serialization.
	 */
	virtual void MarkScriptSerializationStart(const UObject* Obj) { }

	/**
	 * Called when an object stops serializing property data using script serialization.
	 */
	virtual void MarkScriptSerializationEnd(const UObject* Obj) { }

#if UE_INTERNAL_LINKER_BOUNDS
	/**
	 * Update the bounds the archive should be reading from, this is meant to be a number
	 * of expected bytes from the current archive position (established by Tell()).
	 * Bounds are expected to be recursively narrowed by the FArchiveBoundsScope, with outer
	 * bounds restored by inner scopes upon scope exit. This is currently only used in 
	 * a UE_INTERNAL_LINKER_BOUNDS environment and implemented only by
	 * FLinkerLoad, but it could be generally useful.
	 */
	[[nodiscard]] CORE_API FArchiveBoundsScope SetBounds(int64_t Size);
#endif // UE_INTERNAL_LINKER_BOUNDS

	/**
	 * Called to register a reference to a specific name value, of type TypeObject (UEnum or UStruct normally). Const so it can be called from PostSerialize
	 */
	virtual void MarkSearchableName(const TObjectPtr<const UObject>& TypeObject, const FName& ValueName) const { }

	using FArchiveState::GetArchetypeFromLoader;

private:
	CORE_API void VARARGS LogfImpl(const TCHAR* Fmt, ...);
	
#if UE_INTERNAL_LINKER_BOUNDS
	// impl functions for archives that want to support bounds checking
	// this default implementation does nothing:
	CORE_API virtual void SetBoundsImpl(FArchiveBounds NewBounds);
	CORE_API virtual FArchiveBounds GetBoundsImpl() const;
	friend struct FArchiveBoundsScope;
#endif // UE_INTERNAL_LINKER_BOUNDS

public:
	// Logf implementation for convenience.
	template <typename FmtType, typename... Types>
	void Logf(const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FArchive::Logf");

		LogfImpl(Fmt, Args...);
	}

	using FArchiveState::UEVer;
	using FArchiveState::LicenseeUEVer;
	using FArchiveState::EngineVer;
	using FArchiveState::EngineNetVer;
	using FArchiveState::GameNetVer;

	/**
	 * Registers the custom version to the archive.  This is used to inform the archive that custom version information is about to be stored.
	 * There is no effect when the archive is being loaded from.
	 *
	 * @param Guid The guid of the custom version.  This must have previously been registered with FCustomVersionRegistration.
	 */
	CORE_API virtual void UsingCustomVersion(const struct FGuid& Guid);

	using FArchiveState::CustomVer;

	/**
	 * Returns a pointer to an archive that represents the same data that the current archive covers, but that can be cached and reused later
	 * In the case of standard archives, this function will just return a pointer to itself. If the archive is actually a temporary proxy to
	 * another archive, and has a shorter lifecycle than the source archive, it should return either a pointer to the underlying archive, or
	 * if the data becomes inaccessible when the proxy object disappears (as is the case with text format archives) then nullptr
	 */
	virtual FArchive* GetCacheableArchive()
	{
		return this;
	}

	using FArchiveState::IsLoading;
	using FArchiveState::IsLoadingFromCookedPackage;
	using FArchiveState::IsSaving;
	using FArchiveState::IsTransacting;
	using FArchiveState::IsTextFormat;
	using FArchiveState::WantBinaryPropertySerialization;
	using FArchiveState::UseUnversionedPropertySerialization;
	using FArchiveState::IsForcingUnicode;
	using FArchiveState::IsPersistent;
	using FArchiveState::GetError;
	using FArchiveState::IsError;
	using FArchiveState::IsCriticalError;
	using FArchiveState::ContainsCode;
	using FArchiveState::ContainsMap;
	using FArchiveState::RequiresLocalizationGather;
	using FArchiveState::ForceByteSwapping;
	using FArchiveState::IsSerializingDefaults;
	using FArchiveState::IsIgnoringArchetypeRef;
	using FArchiveState::DoDelta;
	using FArchiveState::DoIntraPropertyDelta;
	using FArchiveState::IsIgnoringOuterRef;
	using FArchiveState::IsIgnoringClassGeneratedByRef;
	using FArchiveState::IsIgnoringClassRef;
	using FArchiveState::IsAllowingLazyLoading;
	using FArchiveState::IsObjectReferenceCollector;
	using FArchiveState::IsModifyingWeakAndStrongReferences;
	using FArchiveState::IsCountingMemory;
	using FArchiveState::GetPortFlags;
	using FArchiveState::HasAnyPortFlags;
	using FArchiveState::HasAllPortFlags;
	using FArchiveState::GetDebugSerializationFlags;
	using FArchiveState::ShouldSkipBulkData;
	using FArchiveState::GetMaxSerializeSize;
	using FArchiveState::GetCustomVersions;
	using FArchiveState::SetCustomVersions;
	using FArchiveState::ResetCustomVersions;
	using FArchiveState::SetCustomVersion;
	using FArchiveState::SetByteSwapping;
	using FArchiveState::SetPortFlags;
	using FArchiveState::SetDebugSerializationFlags;
	using FArchiveState::IsFilterEditorOnly;
	using FArchiveState::SetFilterEditorOnly;
	using FArchiveState::IsSavingOptionalObject;
	using FArchiveState::SetSavingOptionalObject;
	using FArchiveState::IsSaveGame;
	using FArchiveState::IsNetArchive;
	using FArchiveState::IsCooking;
	using FArchiveState::CookingTarget;
	using FArchiveState::SetSavePackageData;
	using FArchiveState::GetSavePackageData;
	using FArchiveState::GetCookContext;
	using FArchiveState::UseToResolveEnumerators;
	using FArchiveState::ShouldSkipProperty;
	using FArchiveState::SetSerializedProperty;
	using FArchiveState::GetSerializedProperty;
	using FArchiveState::GetSerializedPropertyChain;
	using FArchiveState::SetSerializedPropertyChain;

	/**
	 * Push a property that is currently being serialized onto the property stack
	 * 
	 * @param InProperty			Pointer to the property that is currently being serialized
	 * @param bIsEditorOnlyProperty True if the property is editor only (call FProperty::IsEditorOnlyProperty to work this out, as the archive can't since it can't access CoreUObject types)
	 */
	CORE_API virtual void PushSerializedProperty(class FProperty* InProperty, const bool bIsEditorOnlyProperty);

	/**
	 * Pop a property that was previously being serialized off the property stack
	 * 
	 * @param InProperty			Pointer to the property that was previously being serialized
	 * @param bIsEditorOnlyProperty True if the property is editor only (call FProperty::IsEditorOnlyProperty to work this out, as the archive can't since it can't access CoreUObject types)
	 */
	CORE_API virtual void PopSerializedProperty(class FProperty* InProperty, const bool bIsEditorOnlyProperty);

#if WITH_EDITORONLY_DATA
	using FArchiveState::IsEditorOnlyPropertyOnTheStack;
#endif

	/**
	 * Adds external read dependency 
	 *
	 * @return true if dependency has been added, false if Archive does not support them
	 */
	virtual bool AttachExternalReadDependency(FExternalReadCallback& ReadCallback) { return false; }


#if USE_STABLE_LOCALIZATION_KEYS
	using FArchiveState::SetLocalizationNamespace;
	using FArchiveState::GetLocalizationNamespace;
#endif // USE_STABLE_LOCALIZATION_KEYS

	/** Resets all of the base archive members. */
	using FArchiveState::Reset;

public:
#if DEVIRTUALIZE_FLinkerLoad_Serialize
	template<SIZE_T Size>
	inline bool FastPathLoad(void* InDest)
	{
		const uint8* RESTRICT Src = ActiveFPLB->StartFastPathLoadBuffer;
		if (Src + Size <= ActiveFPLB->EndFastPathLoadBuffer)
		{
			if constexpr (Size == 2)
			{
				uint16 * RESTRICT Dest = (uint16 * RESTRICT)InDest;
				*Dest = FPlatformMemory::ReadUnaligned<uint16>(Src);
			}
			else if constexpr (Size == 4)
			{
				uint32 * RESTRICT Dest = (uint32 * RESTRICT)InDest;
				*Dest = FPlatformMemory::ReadUnaligned<uint32>(Src);
			}
			else if constexpr (Size == 8)
			{
				uint64 * RESTRICT Dest = (uint64 * RESTRICT)InDest;
				*Dest = FPlatformMemory::ReadUnaligned<uint64>(Src);
			}
			else
			{
				uint8 * RESTRICT Dest = (uint8 * RESTRICT)InDest;
				for (SIZE_T Index = 0; Index < Size; Index++)
				{
					Dest[Index] = Src[Index];
				}
			}
			ActiveFPLB->StartFastPathLoadBuffer += Size;
			return true;
		}
		return false;
	}

	//@todoio FArchive is really a horrible class and the way it is proxied by FLinkerLoad is double terrible. It makes the fast path really hacky and slower than it would need to be.
	using FArchiveState::ActiveFPLB;
	using FArchiveState::InlineFPLB;

#else
	template<SIZE_T Size>
	UE_FORCEINLINE_HINT bool FastPathLoad(void* InDest)
	{
		return false;
	}
#endif

private:
	// Used internally only to control the amount of generated code/type under control.
	template<typename T>
	FArchive& ByteOrderSerialize(T& Value)
	{
		static_assert(!TIsSigned<T>::Value, "To reduce the number of template instances, cast 'Value' to a uint16&, uint32& or uint64& prior to the call or use ByteOrderSerialize(void*, int32).");

		if (!IsByteSwapping()) // Most likely case (hot path)
		{
			Serialize(&Value, sizeof(T));
			return *this;
		}
		return SerializeByteOrderSwapped(Value); // Slowest and unlikely path (but fastest than SerializeByteOrderSwapped(void*, int32)).
	}

	// Not inlined to keep ByteOrderSerialize(), small and fast.
	CORE_API FArchive& SerializeByteOrderSwapped(void* V, int32 Length);
	CORE_API FArchive& SerializeByteOrderSwapped(uint16& Value);
	CORE_API FArchive& SerializeByteOrderSwapped(uint32& Value);
	CORE_API FArchive& SerializeByteOrderSwapped(uint64& Value);
#if PLATFORM_COMPILER_DISTINGUISHES_INT_AND_LONG
	FArchive& SerializeByteOrderSwapped(unsigned long& Value)
	{
		static_assert(sizeof(unsigned long) == sizeof(uint64), "Wrong unsigned long size assuption.");
		return SerializeByteOrderSwapped(reinterpret_cast<uint64&>(Value));
	}
#endif

private:
	using FArchiveState::ArIsLoading;
	using FArchiveState::ArIsLoadingFromCookedPackage;
	using FArchiveState::ArIsSaving;
	using FArchiveState::ArIsTransacting;
	using FArchiveState::ArIsTextFormat;
	using FArchiveState::ArWantBinaryPropertySerialization;
	using FArchiveState::ArUseUnversionedPropertySerialization;
	using FArchiveState::ArForceUnicode;
	using FArchiveState::ArIsPersistent;

public:
	using FArchiveState::ArIsError;
	using FArchiveState::ArIsCriticalError;
	using FArchiveState::ArContainsCode;
	using FArchiveState::ArContainsMap;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	using FArchiveState::ArRequiresLocalizationGather;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	using FArchiveState::ArForceByteSwapping;
	using FArchiveState::ArIgnoreArchetypeRef;
	using FArchiveState::ArNoDelta;
	using FArchiveState::ArNoIntraPropertyDelta;
	using FArchiveState::ArIgnoreOuterRef;
	using FArchiveState::ArIgnoreClassGeneratedByRef;
	using FArchiveState::ArIgnoreClassRef;
	using FArchiveState::ArAllowLazyLoading;
	using FArchiveState::ArIsObjectReferenceCollector;
	using FArchiveState::ArIsModifyingWeakAndStrongReferences;
	using FArchiveState::ArIsCountingMemory;
	using FArchiveState::ArShouldSkipBulkData;
	using FArchiveState::ArIsFilterEditorOnly;
	using FArchiveState::ArIsSavingOptionalObject;
	using FArchiveState::ArIsSaveGame;
	using FArchiveState::ArIsNetArchive;
	using FArchiveState::ArUseCustomPropertyList;
	using FArchiveState::ArSerializingDefaults;
	using FArchiveState::ArPortFlags;
	using FArchiveState::ArMaxSerializeSize;
	using FArchiveState::ArMergeOverrides;
	using FArchiveState::ArPreserveArrayElements;

public:
	using FArchiveState::SetIsLoading;
	using FArchiveState::SetIsLoadingFromCookedPackage;
	using FArchiveState::SetIsSaving;
	using FArchiveState::SetIsTransacting;
	using FArchiveState::SetIsTextFormat;
	using FArchiveState::SetWantBinaryPropertySerialization;
	using FArchiveState::SetUseUnversionedPropertySerialization;
	using FArchiveState::SetForceUnicode;
	using FArchiveState::SetIsPersistent;
	using FArchiveState::SetUEVer;
	using FArchiveState::SetLicenseeUEVer;
	using FArchiveState::SetEngineVer;
	using FArchiveState::SetEngineNetVer;
	using FArchiveState::SetGameNetVer;
	using FArchiveState::ShouldSkipCompilingAssets;
	using FArchiveState::SetShouldSkipUpdateCustomVersion;
	using FArchiveState::ShouldSkipUpdateCustomVersion;
	using FArchiveState::SetMigratingRemoteObjects;
	using FArchiveState::IsMigratingRemoteObjects;
	using FArchiveState::GetMigrationContext;

	
private:
	using FArchiveState::ArUEVer;
	using FArchiveState::ArLicenseeUEVer;
	using FArchiveState::ArEngineVer;
	using FArchiveState::CustomVersionContainer;
protected:
	using FArchiveState::SetShouldSkipCompilingAssets;
public:
	/** Custom property list attribute. If the flag below is set, only these properties will be iterated during serialization. If NULL, then no properties will be iterated. */
	using FArchiveState::ArCustomPropertyList;

	class FScopeSetDebugSerializationFlags
	{
	private:
#if WITH_EDITOR
		uint32 PreviousFlags;
		FArchive& Ar;
#endif
	public:
		/**
		 * Initializes an object which will set flags for the scope of this code
		 * 
		 * @param NewFlags new flags to set 
		 * @param Remove should we add these flags or remove them default is to add
		 */
#if WITH_EDITOR
		FScopeSetDebugSerializationFlags(FArchive& InAr, uint32 NewFlags, bool Remove = false)
			: Ar(InAr)
		{

			PreviousFlags = Ar.GetDebugSerializationFlags();
			if (Remove)
			{
				Ar.SetDebugSerializationFlags( PreviousFlags & ~NewFlags);
			}
			else
			{
				Ar.SetDebugSerializationFlags( PreviousFlags | NewFlags);
			}

		}
		~FScopeSetDebugSerializationFlags()
		{

			Ar.SetDebugSerializationFlags( PreviousFlags);
		}
#else
		FScopeSetDebugSerializationFlags(FArchive& InAr, uint32 NewFlags, bool Remove = false)
		{}
		~FScopeSetDebugSerializationFlags() = default;
#endif
	};

#if WITH_EDITOR
	/** Custom serialization modifier flags can be used for anything */
	using FArchiveState::ArDebugSerializationFlags;
	
	/** Debug stack storage if you want to add data to the archive for usage further down the serialization stack this should be used in conjunction with the FScopeAddDebugData struct */
	CORE_API virtual void PushDebugDataString(const FName& DebugData);
	virtual void PopDebugDataString() { }

	class FScopeAddDebugData
	{
	private:
		FArchive& Ar;
	public:
		CORE_API FScopeAddDebugData(FArchive& InAr, const FName& DebugData);

		~FScopeAddDebugData()
		{
			Ar.PopDebugDataString();
		}
	};
#endif

	/** Seeks to and restores the position of an archive. */
	class FScopeSeekTo
	{
	public:
		FScopeSeekTo(FArchive& InAr, int64 InPos)
			: Ar(InAr)
			, SavedPos(InAr.Tell())
		{
			Ar.Seek(InPos);
		}
		
		~FScopeSeekTo()
		{
			Ar.Seek(SavedPos);
		}

		FScopeSeekTo(const FScopeSeekTo&) = delete;
		FScopeSeekTo& operator=(const FScopeSeekTo&) = delete;

	private:
		FArchive& Ar;
		int64 SavedPos;
	};

	/** Called whilst cooking to provide file region hints to the cooker. */
	virtual void PushFileRegionType(EFileRegionType Type) { }
	virtual void PopFileRegionType() { }

private:
	/** Holds the pointer to the property that is currently being serialized */
	using FArchiveState::SerializedProperty;

	/** Holds the chain of properties that are currently being serialized */
	using FArchiveState::SerializedPropertyChain;

#if USE_STABLE_LOCALIZATION_KEYS
	/**
	 * The localization namespace that this archive should use when serializing text properties.
	 * This is typically the namespace used by the package being serialized (if serializing a package, or an object within a package).
	 * Stored as a pointer to a heap-allocated string because of a dependency between TArray (thus FString) and FArchive; null should be treated as an empty string.
	 */
	using FArchiveState::LocalizationNamespacePtr;
#endif // USE_STABLE_LOCALIZATION_KEYS

	/**
	 * Indicates if the custom versions container is in a 'reset' state.  This will be used to defer the choice about how to
	 * populate the container until it is needed, where the read/write state will be known.
	 */
	using FArchiveState::bCustomVersionsAreReset;
};




/**
 * Template for archive constructors.
 */
template<class T> T Arctor(FArchive& Ar)
{
	T Tmp;
	Ar << Tmp;

	return Tmp;
}

/**
* Serializes an enumeration value from or into an archive.
*
* @param Ar The archive to serialize from or to.
* @param Value The value to serialize.
*/
template<class TEnum>
inline FArchive& operator<<(FArchive& Ar, TEnumAsByte<TEnum>& Value)
{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
	if (!Ar.FastPathLoad<sizeof(Value)>(&Value))
#endif
	{
		Ar.Serialize(&Value, 1);
	}
	return Ar;
}

/**
* Serializes enum classes as their underlying type.
*
* @param Ar The archive to serialize from or to.
* @param Value The value to serialize.
*/
template <
	typename EnumType,
	std::enable_if_t<TIsEnumClass<EnumType>::Value, int> = 0
>
UE_FORCEINLINE_HINT FArchive& operator<<(FArchive& Ar, EnumType& Value)
{
	return Ar << (__underlying_type(EnumType)&)Value;
}

