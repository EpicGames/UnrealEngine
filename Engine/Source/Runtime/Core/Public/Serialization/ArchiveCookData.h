// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class ITargetPlatform;
struct FArchiveCookContext;

// Holds archive data only relevant for cooking archives. Fill this out
// as part of FSavePackageArgs.
struct FArchiveCookData
{
	const ITargetPlatform& TargetPlatform;
	FArchiveCookContext& CookContext;
	FArchiveCookData(const ITargetPlatform& InTargetPlatform, FArchiveCookContext& InCookContext) :
		TargetPlatform(InTargetPlatform),
		CookContext(InCookContext)
	{}
};
