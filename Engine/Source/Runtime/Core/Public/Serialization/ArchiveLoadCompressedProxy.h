// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Misc/Compression.h"
#include "Misc/CompressionFlags.h"
#include "Serialization/Archive.h"
#include "UObject/NameTypes.h"

/*----------------------------------------------------------------------------
	FArchiveLoadCompressedProxy.
----------------------------------------------------------------------------*/

/**
 * FArchive Proxy to transparently load compressed data from an array.
 */
class FArchiveLoadCompressedProxy : public FArchive
{
public:
	CORE_API FArchiveLoadCompressedProxy(const TArray<uint8>& InCompressedData, FName CompressionFormat, ECompressionFlags InCompressionFlags=COMPRESS_NoFlags);

	/** Destructor, freeing temporary memory. */
	CORE_API virtual ~FArchiveLoadCompressedProxy();

	/**
	 * Serializes data from archive. This function is called recursively and determines where to serialize
	 * from and how to do so based on internal state.
	 *
	 * @param	Data	Pointer to serialize to
	 * @param	Count	Number of bytes to read
	 */
	CORE_API virtual void Serialize( void* Data, int64 Count );

	/**
	 * Seeks to the passed in position in the stream. This archive only supports forward seeking
	 * and implements it by serializing data till it reaches the position.
	 */
	CORE_API virtual void Seek( int64 InPos );

	/**
	 * @return current position in uncompressed stream in bytes.
	 */
	CORE_API virtual int64 Tell();

private:
	/**
	 * Flushes tmp data to array.
	 */
	void DecompressMoreData();

	/** Array to write compressed data to.						*/
	const TArray<uint8>&	CompressedData;
	/** Current index into compressed data array.				*/
	int32			CurrentIndex;
	/** Pointer to start of temporary buffer.					*/
	uint8*			TmpDataStart;
	/** Pointer to end of temporary buffer.						*/
	uint8*			TmpDataEnd;
	/** Pointer to current position in temporary buffer.		*/
	uint8*			TmpData;
	/** Whether to serialize from temporary buffer of array.	*/
	bool			bShouldSerializeFromArray;
	/** Number of raw (uncompressed) bytes serialized.			*/
	int64			RawBytesSerialized;
	/** Compression method										*/
	FName			CompressionFormat;
	/** Flags used for compression.								*/
	ECompressionFlags CompressionFlags;
};
