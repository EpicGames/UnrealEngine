// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Internationalization/TextNamespaceFwd.h"
#include "Serialization/Archive.h"

class FBulkData;
struct FBulkDataSerializationParams;
struct FCompressedChunk;

/**
 * Base class for archive proxies.
 *
 * Archive proxies are archive types that modify the behavior of another archive type.
 */
class FArchiveProxy : public FArchive
{
public:
	/**
	 * Creates and initializes a new instance of the archive proxy.
	 *
	 * @param InInnerArchive The inner archive to proxy.
	 */
	CORE_API FArchiveProxy(FArchive& InInnerArchive);
	CORE_API ~FArchiveProxy();

	// Non-copyable
	FArchiveProxy(FArchiveProxy&&) = delete;
	FArchiveProxy(const FArchiveProxy&) = delete;
	FArchiveProxy& operator=(FArchiveProxy&&) = delete;
	FArchiveProxy& operator=(const FArchiveProxy&) = delete;

	virtual FArchive& operator<<(FName& Value) override
	{
		InnerArchive << Value;
		return *this;
	}

	virtual FArchive& operator<<(FText& Value) override
	{
		InnerArchive << Value;
		return *this;
	}

	virtual FArchive& operator<<(UObject*& Value) override
	{
		InnerArchive << Value;
		return *this;
	}

	virtual FArchive& operator<<(FObjectPtr& Value) override
	{
		InnerArchive << Value;
		return *this;
	}

	virtual FArchive& operator<<(FLazyObjectPtr& Value) override
	{
		InnerArchive << Value;
		return *this;
	}

	virtual FArchive& operator<<(FSoftObjectPath& Value) override
	{
		InnerArchive << Value;
		return *this;
	}

	virtual FArchive& operator<<(FSoftObjectPtr& Value) override
	{
		InnerArchive << Value;
		return *this;
	}

	virtual FArchive& operator<<(FWeakObjectPtr& Value) override
	{
		InnerArchive << Value;
		return *this;
	}

	virtual FArchive& operator<<(FField*& Value) override
	{
		InnerArchive << Value;
		return *this;
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual FArchive& operator<<(Verse::VCell*& Value)
	{
		InnerArchive << Value;
		return *this;
	}
#endif

	virtual void Serialize(void* V, int64 Length) override
	{
		InnerArchive.Serialize(V, Length);
	}

	virtual void SerializeBits(void* Bits, int64 LengthBits) override
	{
		InnerArchive.SerializeBits(Bits, LengthBits);
	}

	virtual void SerializeInt(uint32& Value, uint32 Max) override
	{
		InnerArchive.SerializeInt(Value, Max);
	}

	virtual void SerializeIntPacked(uint32& Value) override
	{
		InnerArchive.SerializeIntPacked(Value);
	}

	virtual void Preload(UObject* Object) override
	{
		InnerArchive.Preload(Object);
	}

	virtual void CountBytes(SIZE_T InNum, SIZE_T InMax) override
	{
		InnerArchive.CountBytes(InNum, InMax);
	}

	CORE_API virtual FString GetArchiveName() const override;

	virtual FLinker* GetLinker() override
	{
		return InnerArchive.GetLinker();
	}

#if USE_STABLE_LOCALIZATION_KEYS
	CORE_API virtual void SetLocalizationNamespace(const FString& InLocalizationNamespace) override;
	CORE_API virtual FString GetLocalizationNamespace() const override;
#endif // USE_STABLE_LOCALIZATION_KEYS

	virtual int64 Tell() override
	{
		return InnerArchive.Tell();
	}

	virtual int64 TotalSize() override
	{
		return InnerArchive.TotalSize();
	}

	virtual bool AtEnd() override
	{
		return InnerArchive.AtEnd();
	}

	virtual void Seek(int64 InPos) override
	{
		InnerArchive.Seek(InPos);
	}

	virtual void AttachBulkData(UObject* Owner, FBulkData* BulkData) override
	{
		InnerArchive.AttachBulkData(Owner, BulkData);
	}

	virtual void DetachBulkData(FBulkData* BulkData, bool bEnsureBulkDataIsLoaded) override
	{
		InnerArchive.DetachBulkData(BulkData, bEnsureBulkDataIsLoaded);
	}

	virtual bool SerializeBulkData(FBulkData& BulkData, const FBulkDataSerializationParams& Params) override
	{
		return InnerArchive.SerializeBulkData(BulkData, Params);
	}

	virtual bool Precache(int64 PrecacheOffset, int64 PrecacheSize) override
	{
		return InnerArchive.Precache(PrecacheOffset, PrecacheSize);
	}

	virtual bool SetCompressionMap(TArray<FCompressedChunk>* CompressedChunks, ECompressionFlags CompressionFlags) override
	{
		return InnerArchive.SetCompressionMap(CompressedChunks, CompressionFlags);
	}

	virtual void Flush() override
	{
		InnerArchive.Flush();
	}

	virtual bool Close() override
	{
		return InnerArchive.Close();
	}

	virtual void MarkScriptSerializationStart(const UObject* Obj) override
	{
		InnerArchive.MarkScriptSerializationStart(Obj);
	}

	virtual void MarkScriptSerializationEnd(const UObject* Obj) override
	{
		InnerArchive.MarkScriptSerializationEnd(Obj);
	}

	virtual const FCustomVersionContainer& GetCustomVersions() const override
	{
		return InnerArchive.GetCustomVersions();
	}

	virtual void SetCustomVersions(const FCustomVersionContainer& NewVersions) override
	{
		InnerArchive.SetCustomVersions(NewVersions);
	}

	virtual void ResetCustomVersions() override
	{
		InnerArchive.ResetCustomVersions();
	}

	virtual void MarkSearchableName(const TObjectPtr<const UObject>& TypeObject, const FName& ValueName) const override
	{
		InnerArchive.MarkSearchableName(TypeObject, ValueName);
	}

	virtual UObject* GetArchetypeFromLoader(const UObject* Obj) override
	{
		return InnerArchive.GetArchetypeFromLoader(Obj);
	}

	virtual bool AttachExternalReadDependency(FExternalReadCallback& ReadCallback) override
	{
		return InnerArchive.AttachExternalReadDependency(ReadCallback);
	}

	virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
	{
		return InnerArchive.ShouldSkipProperty(InProperty);
	}

	virtual bool UseToResolveEnumerators() const override
	{
		return InnerArchive.UseToResolveEnumerators();
	}

	virtual void FlushCache() override
	{
		InnerArchive.FlushCache();
	}

	virtual void ForceBlueprintFinalization() override
	{
		InnerArchive.ForceBlueprintFinalization();
	}

	virtual void SetDebugSerializationFlags(uint32 InCustomFlags) override
	{
		FArchive::SetDebugSerializationFlags(InCustomFlags);
		InnerArchive.SetDebugSerializationFlags(InCustomFlags);
	}

	virtual void SetFilterEditorOnly(bool InFilterEditorOnly) override
	{
		FArchive::SetFilterEditorOnly(InFilterEditorOnly);
		InnerArchive.SetFilterEditorOnly(InFilterEditorOnly);
	}

#if WITH_EDITOR
	virtual void PushDebugDataString(const FName& DebugData) override
	{
		InnerArchive.PushDebugDataString(DebugData);
	}
	virtual void PopDebugDataString() override
	{
		InnerArchive.PopDebugDataString();
	}
#endif

	inline void SetSerializedProperty(FProperty* InProperty) override
	{
		FArchive::SetSerializedProperty(InProperty);
		InnerArchive.SetSerializedProperty(InProperty);
	}

	void SetSerializedPropertyChain(const FArchiveSerializedPropertyChain* InSerializedPropertyChain, class FProperty* InSerializedPropertyOverride = nullptr) override
	{
		FArchive::SetSerializedPropertyChain(InSerializedPropertyChain, InSerializedPropertyOverride);
		InnerArchive.SetSerializedPropertyChain(InSerializedPropertyChain, InSerializedPropertyOverride);
	}

	/** Pushes editor-only marker to the stack of currently serialized properties */
	virtual inline void PushSerializedProperty(class FProperty* InProperty, const bool bIsEditorOnlyProperty) override
	{
		FArchive::PushSerializedProperty(InProperty, bIsEditorOnlyProperty);
		InnerArchive.PushSerializedProperty(InProperty, bIsEditorOnlyProperty);
	}
	/** Pops editor-only marker from the stack of currently serialized properties */
	virtual inline void PopSerializedProperty(class FProperty* InProperty, const bool bIsEditorOnlyProperty) override
	{
		FArchive::PopSerializedProperty(InProperty, bIsEditorOnlyProperty);
		InnerArchive.PopSerializedProperty(InProperty, bIsEditorOnlyProperty);
	}
#if WITH_EDITORONLY_DATA
	/** Returns true if the stack of currently serialized properties contains an editor-only property */
	virtual UE_FORCEINLINE_HINT bool IsEditorOnlyPropertyOnTheStack() const override
	{
		return InnerArchive.IsEditorOnlyPropertyOnTheStack();
	}
#endif

	virtual UE_FORCEINLINE_HINT bool IsProxyOf(FArchive* InOther) const override
	{
		return InOther == this || InOther == &InnerArchive || InnerArchive.IsProxyOf(InOther);
	}

	virtual FArchive* GetCacheableArchive() override
	{
		return InnerArchive.GetCacheableArchive();
	}

	virtual ::FArchiveState& GetInnermostState() override
	{
		return InnerArchive.GetInnermostState();
	}

protected:

	/** Holds the archive that this archive is a proxy to. */
	FArchive& InnerArchive;
};
