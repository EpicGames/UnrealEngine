// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Misc/Compression.h"
#include "Misc/CompressionFlags.h"
#include "Serialization/Archive.h"
#include "UObject/NameTypes.h"

/*----------------------------------------------------------------------------
	FArchiveSaveCompressedProxy.
----------------------------------------------------------------------------*/

/**
 * FArchive Proxy to transparently write out compressed data to an array.
 */
class FArchiveSaveCompressedProxy : public FArchive
{	
public:
	/** 
	 * Constructor, initializing all member variables and allocating temp memory.
	 *
	 * @param	InCompressedData [ref]	Array of bytes that is going to hold compressed data
	 * @param	InCompressionFormat		Compression format to use
	 */	
	CORE_API FArchiveSaveCompressedProxy( TArray<uint8>& InCompressedData, FName InCompressionFormat, ECompressionFlags InCompressionFlags=COMPRESS_None);

	/** Destructor, flushing array if needed. Also frees temporary memory. */
	CORE_API virtual ~FArchiveSaveCompressedProxy();

	/**
	 * Flushes tmp data to array.
	 */
	CORE_API virtual void Flush();

	/**
	 * Serializes data to archive. This function is called recursively and determines where to serialize
	 * to and how to do so based on internal state.
	 *
	 * @param	Data	Pointer to serialize to
	 * @param	Count	Number of bytes to read
	 */
	CORE_API virtual void Serialize( void* Data, int64 Count );

	/**
	 * Seeking is only implemented internally for writing out compressed data and asserts otherwise.
	 * 
	 * @param	InPos	Position to seek to
	 */
	CORE_API virtual void Seek( int64 InPos );

	/**
	 * @return current position in uncompressed stream in bytes.
	 */
	CORE_API virtual int64 Tell();

private:
	/** Array to write compressed data to.					*/
	TArray<uint8>&	CompressedData;
	/** Current index in array.								*/
	int32				CurrentIndex;
	/** Pointer to start of temporary buffer.				*/
	uint8*			TmpDataStart;
	/** Pointer to end of temporary buffer.					*/
	uint8*			TmpDataEnd;
	/** Pointer to current position in temporary buffer.	*/
	uint8*			TmpData;
	/** Whether to serialize to temporary buffer of array.	*/
	bool			bShouldSerializeToArray;
	/** Number of raw (uncompressed) bytes serialized.		*/
	int64		RawBytesSerialized;
	/** Format to use for compression.						*/
	FName  CompressionFormat;
	/** Flags to use for compression.						*/
	ECompressionFlags CompressionFlags;
};

