// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FObjectSavePackageSerializeContext;
class ITargetPlatform;
struct FArchiveCookContext;

/** Holds archive data only relevant for archives used during SavePackage. */
struct FArchiveSavePackageData
{
	FArchiveSavePackageData(FObjectSavePackageSerializeContext& InSavePackageContext,
		const ITargetPlatform* InTargetPlatform, FArchiveCookContext* InCookContext)
		: SavePackageContext(InSavePackageContext)
		, TargetPlatform(InTargetPlatform)
		, CookContext(InCookContext)
	{

	}
	FObjectSavePackageSerializeContext& SavePackageContext;
	/** Non-null if a cook save. Null if an editor save. */
	const ITargetPlatform* TargetPlatform = nullptr;
	/** Non-null if a cook save. Null if an editor save. */
	FArchiveCookContext* CookContext = nullptr;
};
