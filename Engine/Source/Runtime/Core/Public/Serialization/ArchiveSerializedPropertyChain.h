// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Misc/NotNull.h"
#include "Templates/Function.h"

#if WITH_EDITORONLY_DATA
namespace UE::IDOHelpers
{
	PRAGMA_DISABLE_INTERNAL_WARNINGS
	UE_INTERNAL extern CORE_API TFunction<bool(TNotNull<FProperty*>)> GIsUnknownPropertyFunc;
	PRAGMA_ENABLE_INTERNAL_WARNINGS
}
#endif

/** Structure containing the stack of properties that are currently being serialized by an archive */
struct FArchiveSerializedPropertyChain
{
public:
	/** Default constructor */
	FArchiveSerializedPropertyChain()
		: SerializedPropertyChainUpdateCount(0)
#if WITH_EDITORONLY_DATA
		, EditorOnlyPropertyStack(0)
		, UnknownPropertyStack(0)
#endif
	{
	}

	/**
	 * Push a property that is currently being serialized
	 * 
	 * @param InProperty			Pointer to the property that is currently being serialized
	 * @param bIsEditorOnlyProperty True if the property is editor only (call FProperty::IsEditorOnlyProperty to work this out, as the archive can't since it can't access CoreUObject types)
	 */
	void PushProperty(class FProperty* InProperty, const bool bIsEditorOnlyProperty)
	{
		check(InProperty);
		SerializedPropertyChain.Push(InProperty);

		IncrementUpdateCount();

#if WITH_EDITORONLY_DATA
		if (bIsEditorOnlyProperty)
		{
			++EditorOnlyPropertyStack;
		}
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		if (UE::IDOHelpers::GIsUnknownPropertyFunc.IsSet() && UE::IDOHelpers::GIsUnknownPropertyFunc(InProperty))
		{
			++UnknownPropertyStack;
		}
		PRAGMA_ENABLE_INTERNAL_WARNINGS
#endif // WITH_EDITORONLY_DATA
	}

	/**
	 * Pop a property that was previously being serialized
	 * 
	 * @param InProperty			Pointer to the property that was previously being serialized
	 * @param bIsEditorOnlyProperty True if the property is editor only (call FProperty::IsEditorOnlyProperty to work this out, as the archive can't since it can't access CoreUObject types)
	 */
	void PopProperty(class FProperty* InProperty, const bool bIsEditorOnlyProperty)
	{
		check(InProperty);
		check(SerializedPropertyChain.Num() > 0 && SerializedPropertyChain.Last() == InProperty);
		SerializedPropertyChain.Pop(EAllowShrinking::No);

		IncrementUpdateCount();

#if WITH_EDITORONLY_DATA
		if (bIsEditorOnlyProperty)
		{
			--EditorOnlyPropertyStack;
			check(EditorOnlyPropertyStack >= 0);
		}
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		if (UE::IDOHelpers::GIsUnknownPropertyFunc.IsSet() && UE::IDOHelpers::GIsUnknownPropertyFunc(InProperty))
		{
			--UnknownPropertyStack;
		}
		PRAGMA_ENABLE_INTERNAL_WARNINGS
#endif // WITH_EDITORONLY_DATA
	}

	/**
	 * Get the property at the given index on the stack
	 * @note This index is in stack order, so the 0th index with be the leaf property on the stack
	 */
	class FProperty* GetPropertyFromStack(const int32 InStackIndex) const
	{
		return SerializedPropertyChain.Last(InStackIndex);
	}

	/**
	 * Get the property at the given index from the root
	 * @note This index is in array order, so the 0th index with be the root property on the stack
	 */
	class FProperty* GetPropertyFromRoot(const int32 InRootIndex) const
	{
		return SerializedPropertyChain[InRootIndex];
	}

	/**
	 * Get the number of properties currently on the stack
	 */
	int32 GetNumProperties() const
	{
		return SerializedPropertyChain.Num();
	}

	/**
	 * Get the counter for the number of times that SerializedPropertyChain has been updated
	 */
	uint32 GetUpdateCount() const
	{
		return SerializedPropertyChainUpdateCount;
	}

	/**
	 * Check to see whether there are any editor-only properties on the stack
	 */
	bool HasEditorOnlyProperty() const
	{
#if WITH_EDITORONLY_DATA
		return EditorOnlyPropertyStack > 0;
#else
		return false;
#endif
	}

	/**
	 * Check to see whether there are any properties on the stack is unknown
	 */
	bool HasUnknownProperty() const
	{
#if WITH_EDITORONLY_DATA
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		checkf(UE::IDOHelpers::GIsUnknownPropertyFunc.IsSet(), TEXT("There was no function pointer provided to retrieve this information, so the result will not be correctly returned."))
		PRAGMA_ENABLE_INTERNAL_WARNINGS
		return UnknownPropertyStack > 0;
#else // WITH_EDITORONLY_DATA
		return false;
#endif// WITH_EDITORONLY_DATA
	}

	UE_FORCEINLINE_HINT TArray<class FProperty*, TInlineAllocator<8>>::TConstIterator GetRootIterator() const
	{
		return SerializedPropertyChain.CreateConstIterator();
	}
private:
	void IncrementUpdateCount()
	{
		while (++SerializedPropertyChainUpdateCount == 0) {} // Zero is special, don't allow an overflow to stay at zero
	}

	/** Array of properties on the stack */
	TArray<class FProperty*, TInlineAllocator<8>> SerializedPropertyChain;

	/** Counter for the number of times that SerializedPropertyChain has been updated */
	uint32 SerializedPropertyChainUpdateCount;

#if WITH_EDITORONLY_DATA
	/** Non-zero if on the current stack of serialized properties there's an editor-only property */
	int32 EditorOnlyPropertyStack;

	/** Non-zero if on the current stack of serialized properties is a unknown property */
	int32 UnknownPropertyStack;

#endif // WITH_EDITORONLY_DATA
};