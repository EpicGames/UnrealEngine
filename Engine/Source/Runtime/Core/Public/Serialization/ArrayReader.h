// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Serialization/MemoryArchive.h"

class FArrayReader final : public FMemoryArchive, public TArray<uint8>
{
public:
	FArrayReader( bool bIsPersistent=false )
	{
		this->SetIsLoading(true);
		this->SetIsPersistent(bIsPersistent);
	}

	/**
	* Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	* is in when a loading error occurs.
	*
	* This is overridden for the specific Archive Types
	**/
	virtual FString GetArchiveName() const { return TEXT("FArrayReader"); }

	int64 TotalSize()
	{
		return (int64)Num();
	}

	void Serialize(void* Data, int64 Count)
	{
		if (Count && !IsError())
		{
			// Only serialize if we have the requested amount of data
			if (Offset + Count <= Num())
			{
				FMemory::Memcpy(Data, &((*this)[(int32)Offset]), Count);
				Offset += Count;
			}
			else
			{
				SetError();
			}
		}
	}
};
