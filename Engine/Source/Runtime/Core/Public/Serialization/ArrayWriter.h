// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Serialization/BufferArchive.h"

/**
 * Archive objects that are also a TArray. Since FBufferArchive is already the 
 * writer version, we just typedef to the old, outdated name
 */
typedef FBufferArchive FArrayWriter;
