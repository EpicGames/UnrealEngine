// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"

/**
* Base class for serializing bitstreams.
*/
class FBitArchive : public FArchive
{
public:
	/**
	 * Default Constructor
	 */
	CORE_API FBitArchive();

	virtual void SerializeBitsWithOffset( void* Src, int32 SourceBit, int64 LengthBits ) PURE_VIRTUAL(FBitArchive::SerializeBitsWithOffset,);
};
