// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/BitArchive.h"

class FArchive;

CORE_API void appBitsCpy( uint8* Dest, int32 DestBit, uint8* Src, int32 SrcBit, int32 BitCount );

/*-----------------------------------------------------------------------------
	FBitReader.
-----------------------------------------------------------------------------*/

//@TODO: FLOATPRECISION: Public API assumes it can have > 2 GB of bits, but internally uses int32 based TArray for bytes and also has uint32 clamping on bit counts in various places

//
// Reads bitstreams.
//
struct FBitReader : public FBitArchive
{
	friend struct FBitReaderMark;

public:
	CORE_API FBitReader( const uint8* Src = nullptr, int64 CountBits = 0 );
	CORE_API ~FBitReader();

	CORE_API FBitReader(const FBitReader&);
    CORE_API FBitReader& operator=(const FBitReader&);
    CORE_API FBitReader(FBitReader&&);
    CORE_API FBitReader& operator=(FBitReader&&);

	CORE_API void SetData( FBitReader& Src, int64 CountBits );
	CORE_API void SetData( uint8* Src, int64 CountBits );
	CORE_API void SetData( TArray<uint8>&& Src, int64 CountBits );

	/** Equivalent to SetData (reset position, copy from Src into internal buffer), but uses Reset not Empty to avoid a realloc if possible. The internal buffer wiill be able to hold at least the maximum of CountBits and CountBitsWithSlack */
	CORE_API void ResetData(FBitReader& Src, int64 CountBits, int64 CountBitsWithSlack=0);

// Disable false positive buffer overrun warning during pgoprofile linking step
PGO_LINK_DISABLE_WARNINGS
	inline void SerializeBits( void* Dest, int64 LengthBits )
	{
		//@TODO: FLOATPRECISION: This function/class pretends it is 64-bit aware, e.g., in the type of LengthBits and the Pos member, but it is not as appBitsCpy is only 32 bits, the inner Buffer is a 32 bit TArray, etc...
		if ( IsError() || Pos+LengthBits > Num)
		{
			if (!IsError())
			{
				SetOverflowed(LengthBits);
				//UE_LOGF( LogNetSerialization, Error, "FBitReader::SerializeBits: Pos + LengthBits > Num" );
			}
			FMemory::Memzero( Dest, (LengthBits+7)>>3 );
			return;
		}
		//for( int32 i=0; i<LengthBits; i++,Pos++ )
		//	if( Buffer(Pos>>3) & GShift[Pos&7] )
		//		((uint8*)Dest)[i>>3] |= GShift[i&7];
		if( LengthBits == 1 )
		{
			((uint8*)Dest)[0] = 0;
			if( Buffer[(int32)(Pos>>3)] & Shift(Pos&7) )
				((uint8*)Dest)[0] |= 0x01;
			Pos++;
		}
		else if (LengthBits != 0)
		{
			((uint8*)Dest)[((LengthBits+7)>>3) - 1] = 0;
			appBitsCpy((uint8*)Dest, 0, Buffer.GetData(), (int32)Pos, (int32)LengthBits);
			Pos += LengthBits;
		}
	}
PGO_LINK_ENABLE_WARNINGS

	CORE_API virtual void SerializeBitsWithOffset( void* Dest, int32 DestBit, int64 LengthBits ) override;

	// OutValue < ValueMax
	inline void SerializeInt(uint32& OutValue, uint32 ValueMax)
	{
		if (!IsError())
		{
			// Use local variable to avoid Load-Hit-Store
			uint32 Value = 0;
			int64 LocalPos = Pos;
			const int64 LocalNum = Num;

			for (uint32 Mask=1; (Value + Mask) < ValueMax && Mask; Mask *= 2, LocalPos++)
			{
				if (LocalPos >= LocalNum)
				{
					SetOverflowed(LocalPos - Pos);
					break;
				}

				if (Buffer[(int32)(LocalPos >> 3)] & Shift(LocalPos & 7))
				{
					Value |= Mask;
				}
			}

			// Now write back
			Pos = LocalPos;
			OutValue = Value;
		}
	}

	CORE_API virtual void SerializeIntPacked(uint32& Value) override;

	inline uint32 ReadInt(uint32 Max)
	{
		uint32 Value = 0;

		SerializeInt(Value, Max);

		return Value;
	}

	inline uint8 ReadBit()
	{
		uint8 Bit=0;
		//SerializeBits( &Bit, 1 );
		if ( !IsError() )
		{
			int64 LocalPos = Pos;
			const int64 LocalNum = Num;
			if (LocalPos >= LocalNum)
			{
				SetOverflowed(1);
				//UE_LOGF( LogNetSerialization, Error, "FBitReader::SerializeInt: LocalPos >= LocalNum" );
			}
			else
			{
				Bit = !!(Buffer[(int32)(LocalPos>>3)] & Shift(LocalPos&7));
				Pos++;
			}
		}
		return Bit;
	}

	UE_FORCEINLINE_HINT void Serialize( void* Dest, int64 LengthBytes )
	{
		SerializeBits( Dest, LengthBytes*8 );
	}

	UE_FORCEINLINE_HINT uint8* GetData()
	{
		return Buffer.GetData();
	}

	UE_FORCEINLINE_HINT const uint8* GetData() const
	{
		return Buffer.GetData();
	}

	UE_FORCEINLINE_HINT const TArray<uint8>& GetBuffer() const
	{
		return Buffer;
	}

	inline uint8* GetDataPosChecked()
	{
		check(Pos % 8 == 0);
		return &Buffer[(int32)(Pos >> 3)];
	}

	UE_FORCEINLINE_HINT int64 GetBytesLeft() const
	{
		return ((Num - Pos) + 7) >> 3;
	}
	UE_FORCEINLINE_HINT int64 GetBitsLeft() const
	{
		return (Num - Pos);
	}
	UE_FORCEINLINE_HINT bool AtEnd()
	{
		return IsError() || Pos>=Num;
	}
	UE_FORCEINLINE_HINT int64 GetNumBytes() const
	{
		return (Num+7)>>3;
	}
	UE_FORCEINLINE_HINT int64 GetNumBits() const
	{
		return Num;
	}
	UE_FORCEINLINE_HINT int64 GetPosBits() const
	{
		return Pos;
	}
	inline void EatByteAlign()
	{
		int64 PrePos = Pos;

		// Skip over remaining bits in current byte
		Pos = (Pos+7) & (~0x07);

		if ( Pos > Num )
		{
			//UE_LOGF( LogNetSerialization, Error, "FBitReader::EatByteAlign: Pos > Num" );
			SetOverflowed(Pos - PrePos);
		}
	}
	inline void Skip(int32 BitCount)
	{
		if (Pos + BitCount > Num)
		{
			SetOverflowed(Pos);
		}
		else
		{
			Pos += BitCount;
		}
	}

	/**
	 * Marks this bit reader as overflowed.
	 *
	 * @param LengthBits	The number of bits being read at the time of overflow
	 */
	CORE_API void SetOverflowed(int64 LengthBits);

	/** Set the stream at the end */
	void SetAtEnd() { Pos = Num; }

	CORE_API void AppendDataFromChecked( FBitReader& Src );
	CORE_API void AppendDataFromChecked( uint8* Src, uint32 NumBits );
	CORE_API void AppendTo( TArray<uint8> &Buffer );

	/** Counts the in-memory bytes used by this object */
	CORE_API virtual void CountMemory(FArchive& Ar) const;

protected:

	TArray<uint8> Buffer;
	int64 Num;
	int64 Pos;

	/** Copies version information used for network compatibility from Source to this archive */
	CORE_API void SetNetVersionsFromArchive(FArchive& Source);

private:

	UE_FORCEINLINE_HINT uint8 Shift(uint8 Cnt)
	{
		return (uint8)(1<<Cnt);
	}

};


//
// For pushing and popping FBitWriter positions.
//
struct FBitReaderMark
{
public:

	FBitReaderMark()
		: Pos(0)
	{ }

	FBitReaderMark( FBitReader& Reader )
		: Pos(Reader.Pos)
	{ }

	UE_FORCEINLINE_HINT int64 GetPos() const
	{
		return Pos;
	}

	UE_FORCEINLINE_HINT void Pop( FBitReader& Reader )
	{
		Reader.Pos = Pos;
	}

	CORE_API void Copy( FBitReader& Reader, TArray<uint8> &Buffer );

private:

	int64 Pos;
};
