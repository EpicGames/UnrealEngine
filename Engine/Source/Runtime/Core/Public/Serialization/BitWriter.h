// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Build.h"
#include "Serialization/BitArchive.h"

class FArchive;

/*-----------------------------------------------------------------------------
	FBitWriter.
-----------------------------------------------------------------------------*/

//
// Writes bitstreams.
//
struct FBitWriter : public FBitArchive
{
	friend struct FBitWriterMark;

public:
	/** Default constructor. Zeros everything. */
	CORE_API FBitWriter();
	CORE_API virtual ~FBitWriter();

	/**
	 * Constructor using known size the buffer needs to be
	 */
	CORE_API FBitWriter( int64 InMaxBits, bool AllowResize = false );

	CORE_API FBitWriter(const FBitWriter&);
    CORE_API FBitWriter& operator=(const FBitWriter&);
    CORE_API FBitWriter(FBitWriter&&);
    CORE_API FBitWriter& operator=(FBitWriter&&);

	CORE_API virtual void SerializeBits( void* Src, int64 LengthBits ) override;

	CORE_API virtual void SerializeBitsWithOffset( void* Src, int32 SourceBit, int64 LengthBits ) override;

	/**
	 * Serializes a compressed integer - Value must be < Max
	 *
	 * @param Value		The value to serialize, must be < Max
	 * @param Max		The maximum allowed value - good to aim for power-of-two
	 */
	CORE_API virtual void SerializeInt(uint32& Value, uint32 Max) override;

	CORE_API virtual void SerializeIntPacked(uint32& Value) override;

	/**
	 * Serializes the specified Value, but does not bounds check against ValueMax;
	 * instead, it will wrap around if the value exceeds ValueMax (this differs from SerializeInt, which clamps)
	 *
	 * @param Value		The value to serialize
	 * @param ValueMax	The maximum value to write; wraps Value if it exceeds this
	 */
	CORE_API void WriteIntWrapped(uint32 Value, uint32 ValueMax);

	CORE_API void WriteBit( uint8 In );
	CORE_API virtual void Serialize( void* Src, int64 LengthBytes ) override;

	/**
	 * Returns a pointer to our internal buffer.
	 */
	inline uint8* GetData(void)
	{
#if !UE_BUILD_SHIPPING
		// If this happens, your code has insufficient IsError() checks.
		check(!IsError());
#endif

		return Buffer.GetData();
	}

	inline const uint8* GetData(void) const
	{
#if !UE_BUILD_SHIPPING
		// If this happens, your code has insufficient IsError() checks.
		check(!IsError());
#endif

		return Buffer.GetData();
	}

	inline const TArray<uint8>* GetBuffer(void) const
	{
#if !UE_BUILD_SHIPPING
		// If this happens, your code has insufficient IsError() checks.
		check(!IsError());
#endif

		return &Buffer;
	}

	/**
	 * Sets how many bits packet actually contains
	 * Must be not greater than max bits allowed
	 * Useful when writing bytes to buffer manually via pointer from GetData()
	 */
	inline void SetNumBits(int64 NumBits)
	{
		check(NumBits <= Max);
		Num = NumBits;
	}

	/**
	 * Returns the number of bytes written.
	 */
	UE_FORCEINLINE_HINT int64 GetNumBytes(void) const
	{
		return (Num+7)>>3;
	}

	/**
	 * Returns the number of bits written.
	 */
	UE_FORCEINLINE_HINT int64 GetNumBits(void) const
	{
		return Num;
	}

	/**
	 * Returns the number of bits the buffer supports.
	 */
	UE_FORCEINLINE_HINT int64 GetMaxBits(void) const
	{
		return Max;
	}

	/**
	 * Marks this bit writer as overflowed.
	 *
	 * @param LengthBits	The number of bits being written at the time of overflow
	 */
	CORE_API void SetOverflowed(int32 LengthBits);

	/**
	 * Sets whether or not this writer intentionally allows overflows (to avoid logspam)
	 */
	UE_FORCEINLINE_HINT void SetAllowOverflow(bool bInAllow)
	{
		bAllowOverflow = bInAllow;
	}

	inline bool AllowAppend(int64 LengthBits)
	{
		//@TODO: FLOATPRECISION: This class pretends it is 64-bit aware, e.g., in the type of LengthBits and the Num/Max members, but it is not as the inner Buffer is a 32 bit TArray, etc...

		if (Num+LengthBits > Max)
		{
			if (bAllowResize)
			{
				// Resize our buffer. The common case for resizing bitwriters is hitting the max and continuing to add a lot of small segments of data
				// Though we could just allow the TArray buffer to handle the slack and resizing, we would still constantly hit the FBitWriter's max
				// and cause this block to be executed, as well as constantly zeroing out memory inside AddZeroes (though the memory would be allocated
				// in chunks).

				Max = FMath::Max<int64>(Max<<1,Num+LengthBits);
				int64 ByteMax = (Max+7)>>3;
				Buffer.AddZeroed((int32)(ByteMax - Buffer.Num()));
				check((Max+7)>>3== Buffer.Num());
				return true;
			}
			else
			{
				return false;
			}
		}
		return true;
	}

	UE_FORCEINLINE_HINT void SetAllowResize(bool NewResize)
	{
		bAllowResize = NewResize;
	}

	/**
	 * Resets the bit writer back to its initial state
	 */
	CORE_API void Reset() override;

	UE_FORCEINLINE_HINT void WriteAlign()
	{
		Num = ( Num + 7 ) & ( ~0x07 );
	}

	/** Counts the in-memory bytes used by this object */
	CORE_API virtual void CountMemory(FArchive& Ar) const;

private:
	TArray<uint8> Buffer;
	int64   Num;
	int64   Max;
	bool	bAllowResize;

	/** Whether or not overflowing is allowed (overflows silently) */
	bool bAllowOverflow;
};


//
// For pushing and popping FBitWriter positions.
//
struct FBitWriterMark
{
public:

	FBitWriterMark()
		: Overflowed(false)
		, Num(0)
	{ }

	FBitWriterMark( FBitWriter& Writer )
	{
		Init(Writer);
	}

	UE_FORCEINLINE_HINT int64 GetNumBits() const
	{
		return Num;
	}

	inline void Init( FBitWriter& Writer)
	{
		Num = Writer.Num;
		Overflowed = Writer.IsError();
	}

	void Reset()
	{
		Overflowed = false;
		Num = 0;		
	}

	CORE_API void Pop( FBitWriter& Writer );
	CORE_API void Copy( FBitWriter& Writer, TArray<uint8> &Buffer );

	/** Pops the BitWriter back to the start but doesn't clear what was written. */
	UE_FORCEINLINE_HINT void PopWithoutClear( FBitWriter& Writer )
	{
		Writer.Num = Num;
	}

private:

	bool			Overflowed;
	int64			Num;
};
