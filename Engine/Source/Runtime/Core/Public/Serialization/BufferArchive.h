// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Serialization/MemoryWriter.h"

/**
 * Buffer archiver.
 */
template <int IndexSize>
class TBufferArchive : public TMemoryWriter<IndexSize>, public TArray<uint8, TSizedDefaultAllocator<IndexSize>>
{
	static_assert(IndexSize == 32 || IndexSize == 64, "Only 32-bit and 64-bit index sizes supported");

public:
	using ArrayType = TArray<uint8, TSizedDefaultAllocator<IndexSize>>;

	TBufferArchive( bool bIsPersistent = false, const FName InArchiveName = NAME_None )
	: TMemoryWriter<IndexSize>( *static_cast<ArrayType*>(this), bIsPersistent, false, InArchiveName )
	{}
	/**
  	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const
	{
		if constexpr (IndexSize == 64)
		{
			return TEXT("FBufferArchive64 ") + this->ArchiveName.ToString();
		}
		else
		{
			return TEXT("FBufferArchive ") + this->ArchiveName.ToString();
		}
	}
};

// FBufferArchive and FBufferArchive64 are implemented as derived classes rather than aliases
// so that forward declarations will work.

class FBufferArchive : public TBufferArchive<32>
{
	using Super = TBufferArchive<32>;

public:
	using Super::Super;
};

class FBufferArchive64 : public TBufferArchive<64>
{
	using Super = TBufferArchive<64>;

public:
	using Super::Super;
};


template <>
struct TIsContiguousContainer<FBufferArchive>
{
	static constexpr bool Value = TIsContiguousContainer<FBufferArchive::ArrayType>::Value;
};

template <>
struct TIsContiguousContainer<FBufferArchive64>
{
	static constexpr bool Value = TIsContiguousContainer<FBufferArchive64::ArrayType>::Value;
};
