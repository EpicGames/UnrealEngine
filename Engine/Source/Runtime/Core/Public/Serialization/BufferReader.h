// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Serialization/Archive.h"
#include "Containers/UnrealString.h"

/**
* Similar to FMemoryReader, but able to internally
* manage the memory for the buffer.
*/
class FBufferReaderBase : public FArchive
{
public:
	/**
	* Constructor
	*
	* @param Data Buffer to use as the source data to read from
	* @param Size Size of Data
	* @param bInFreeOnClose If true, Data will be FMemory::Free'd when this archive is closed
	* @param bIsPersistent Uses this value for SetIsPersistent()
	* @param bInSHAVerifyOnClose It true, an async SHA verification will be done on the Data buffer (bInFreeOnClose will be passed on to the async task)
	*/
	FBufferReaderBase(void* Data, int64 Size, bool bInFreeOnClose, bool bIsPersistent = false)
		: ReaderData(Data)
		, ReaderPos(0)
		, ReaderSize(Size)
		, bFreeOnClose(bInFreeOnClose)
	{
		this->SetIsLoading(true);
		this->SetIsPersistent(bIsPersistent);
	}

	// Abstract to force implementors to call Close() themselves, since it's a virtual function.
	virtual ~FBufferReaderBase() = 0;

	virtual bool Close() override
	{
		if (bFreeOnClose)
		{
			FMemory::Free(ReaderData);
			ReaderData = nullptr;
		}
		return !IsError();
	}
	void Serialize(void* Data, int64 Num) final
	{
		check(ReaderPos >= 0);
		check(ReaderPos + Num <= ReaderSize);
		FMemory::Memcpy(Data, (uint8*)ReaderData + ReaderPos, Num);
		ReaderPos += Num;
	}
	int64 Tell() final
	{
		return ReaderPos;
	}
	int64 TotalSize() final
	{
		return ReaderSize;
	}
	void Seek(int64 InPos) final 
	{
		check(InPos >= 0);
		check(InPos <= ReaderSize);
		ReaderPos = InPos;
	}
	bool AtEnd() final 
	{
		return ReaderPos >= ReaderSize;
	}
	/**
	* Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	* is in when a loading error occurs.
	*
	* This is overridden for the specific Archive Types
	**/
	virtual FString GetArchiveName() const { return TEXT("FBufferReaderBase"); }
protected:
	void*		ReaderData;
	int64		ReaderPos;
	int64		ReaderSize;
	bool	bFreeOnClose;
};

inline FBufferReaderBase::~FBufferReaderBase()
{
}

/**
 * Similar to FMemoryReader, but able to internally
 * manage the memory for the buffer.
 */
class FBufferReader final : public FBufferReaderBase
{
public:
	/**
	 * Constructor
	 * 
	 * @param Data Buffer to use as the source data to read from
	 * @param Size Size of Data
	 * @param bInFreeOnClose If true, Data will be FMemory::Free'd when this archive is closed
	 * @param bIsPersistent Uses this value for SetIsPersistent()
	 * @param bInSHAVerifyOnClose It true, an async SHA verification will be done on the Data buffer (bInFreeOnClose will be passed on to the async task)
	 */
	FBufferReader( void* Data, int64 Size, bool bInFreeOnClose, bool bIsPersistent = false )
		: FBufferReaderBase(Data, Size, bInFreeOnClose, bIsPersistent)
	{
	}

	virtual ~FBufferReader() override
	{
		Close();
	}

	virtual FString GetArchiveName() const { return TEXT("FBufferReader"); }
};
