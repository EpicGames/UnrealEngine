// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Math/NumericLimits.h"
#include "Serialization/Archive.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "Misc/EnumClassFlags.h"
#include "CoreGlobals.h"

enum class EBufferWriterFlags : uint8
{
	None =				0x0,	// No Flags
	TakeOwnership =		0x1,	// Archive will take ownership of the passed-in memory and free it on destruction
	AllowResize =		0x2,	// Allow overflow by resizing buffer
};

ENUM_CLASS_FLAGS(EBufferWriterFlags);

/**
* Similar to FMemoryWriter, but able to internally
* manage the memory for the buffer.
*/
class FBufferWriter final : public FArchive
{
public:
	/**
	* Constructor
	*
	* @param Data Buffer to use as the source data to read from
	* @param Size Size of Data
	* @param InFlags Additional settings
	*/
	FBufferWriter(void* Data, int64 Size, EBufferWriterFlags InFlags = EBufferWriterFlags::None)
		: WriterData(Data)
		, WriterPos(0)
		, WriterSize(Size)
		, bFreeOnClose((InFlags & EBufferWriterFlags::TakeOwnership) != EBufferWriterFlags::None)
		, bAllowResize((InFlags & EBufferWriterFlags::AllowResize) != EBufferWriterFlags::None)
	{
		this->SetIsSaving(true);
	}

	~FBufferWriter()
	{
		Close();
	}
	bool Close()
	{
		if (bFreeOnClose)
		{
			FMemory::Free(WriterData);
			WriterData = nullptr;
		}
		return !IsError();
	}
	void Serialize(void* Data, int64 Num)
	{
		const int64 NumBytesToAdd = WriterPos + Num - WriterSize;
		if (NumBytesToAdd > 0)
		{
			if (bAllowResize)
			{
				const int64 NewArrayCount = WriterSize + NumBytesToAdd;
				if (NewArrayCount >= MAX_int32)
				{
					UE_LOGF(LogSerialization, Fatal, "FBufferWriter does not support data larger than 2GB. Archive name: %ls.", *GetArchiveName());
				}

				WriterData = FMemory::Realloc(WriterData, NewArrayCount);
				WriterSize = NewArrayCount;
			}
			else
			{
				UE_LOGF(LogSerialization, Fatal, "FBufferWriter overflowed. Archive name: %ls.", *GetArchiveName());
			}
		}

		check(WriterPos >= 0);
		check((WriterPos + Num) <= WriterSize);
		FMemory::Memcpy((uint8*)WriterData + WriterPos, Data, Num);
		WriterPos += Num;
	}
	int64 Tell()
	{
		return WriterPos;
	}
	int64 TotalSize()
	{
		return WriterSize;
	}
	void Seek(int64 InPos)
	{
		check(InPos >= 0);
		check(InPos <= WriterSize);
		WriterPos = InPos;
	}
	bool AtEnd()
	{
		return WriterPos >= WriterSize;
	}
	template <typename Type>
	Type* SerializeUninitialized()
	{
		check(WriterPos >= 0);
		check((WriterPos + (int64)sizeof(Type)) <= WriterSize);
		Type* Data = reinterpret_cast<Type*>((uint8*)WriterData + WriterPos);
		WriterPos += sizeof(Type);
		return Data;
	}
	/**
	* Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	* is in when a loading error occurs.
	*
	* This is overridden for the specific Archive Types
	**/
	virtual FString GetArchiveName() const { return TEXT("FBufferWriter"); }

	void* GetWriterData()
	{
		return WriterData;
	}
protected:
	void* WriterData;
	int64 WriterPos;
	int64 WriterSize;
	bool bFreeOnClose;
	bool bAllowResize;
};
