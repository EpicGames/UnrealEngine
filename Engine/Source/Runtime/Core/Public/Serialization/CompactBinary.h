// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "IO/IoHash.h"
#include "Memory/MemoryFwd.h"
#include "Memory/MemoryView.h"
#include "Memory/SharedBuffer.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/NotNull.h"
#include "String/BytesToHex.h"
#include "Templates/FunctionWithContext.h"
#include "Templates/UnrealTemplate.h"
#include <type_traits>

/**
 * This file declares a compact binary data format that is compatible with JSON and only slightly
 * more expressive. The format is designed to achieve fairly small encoded sizes while also being
 * efficient to read both sequentially and through random access. An atom of data in this compact
 * binary format is called a field, which can be: an object, an array, a byte string, a character
 * string, or one of several scalar types including integer, floating point, boolean, null, hash,
 * uuid, date/time, time span, object identifier, or attachment reference.
 *
 * An object is a collection of name-field pairs, and an array is a collection of fields. Encoded
 * objects and arrays are both written such that they may be interpreted as a field that can then
 * be cast to an object or array. This attribute means that a blob containing compact binary data
 * is always safe to interpret as a field, which allows for easy validation as described later.
 *
 * A field can be constructed as a view of the underlying memory with FCbFieldView, or a FCbField
 * can used when ownership of the underlying memory is required. An object provides this behavior
 * with FCbObjectView or FCbObject, and an array uses FCbArrayView or FCbArray.
 *
 * It is optimal use the view types when possible, and reference types only when they are needed,
 * to avoid the overhead of the atomic reference counting of the shared buffer.
 *
 * A suite of validation functionality is provided by ValidateCompactBinary and its siblings. The
 * Default mode provides a guarantee that the data can be consumed without a crash. Documentation
 * of the other modes is available on ECbValidateMode.
 *
 * Example:
 *
 * void BeginBuild(FCbObject Params)
 * {
 *     if (FSharedBuffer Data = Storage().Load(Params["Data"].AsBinaryAttachment()))
 *     {
 *         SetData(Data);
 *     }
 *
 *     if (Params["Resize"].AsBool())
 *     {
 *         FCbFieldView MaxWidthField = Params["MaxWidth"]
 *         FCbFieldView MaxHeightField = Params["MaxHeight"];
 *         if (MaxWidthField && MaxHeightField)
 *         {
 *             Resize(MaxWidthField.AsInt32(), MaxHeightField.AsInt32());
 *         }
 *     }
 *
 *     for (FCbFieldView Format : Params.FindView(ANSITEXTVIEW("Formats")))
 *     {
 *         BeginCompress(FName(Format.AsString()));
 *     }
 * }
 */

class FArchive;
class FCbArrayView;
class FCbField;
class FCbFieldIterator;
class FCbFieldView;
class FCbFieldViewIterator;
class FCbObjectId;
class FCbObjectView;
class FCbValue;
struct FDateTime;
struct FGuid;
struct FTimespan;

/** A reference to a function that is used to allocate buffers for compact binary data. */
using FCbBufferAllocator = TNotNull<UE::TFunctionWithContext<FUniqueBuffer (uint64 Size)>>;

/**
 * A reference to a function that is used to allocate memory for compact binary data.
 * The provider of the allocator must manage ownership of the memory for which a view is returned.
 */
using FCbMemoryAllocator = TNotNull<UE::TFunctionWithContext<FMutableMemoryView (uint64 Size)>>;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Field types and flags for FCbField[View].
 *
 * DO NOT CHANGE THE VALUE OF ANY MEMBERS OF THIS ENUM!
 * BACKWARD COMPATIBILITY REQUIRES THAT THESE VALUES BE FIXED!
 * SERIALIZATION USES HARD-CODED CONSTANTS BASED ON THESE VALUES!
 */
enum class ECbFieldType : uint8
{
	/** A field type that does not occur in a valid object. */
	None                             = 0x00,

	/** Null. Value is empty. */
	Null                             = 0x01,

	/**
	 * Object is an array of fields with unique non-empty names.
	 *
	 * Value is a VarUInt byte count for the encoded fields, followed by the fields.
	 */
	Object                           = 0x02,
	/**
	 * UniformObject is an array of fields with the same field types and unique non-empty names.
	 *
	 * Value is a VarUInt byte count for the encoded fields, followed by the field type, followed
	 * by the fields encoded with no type.
	 */
	UniformObject                    = 0x03,

	/**
	 * Array is an array of fields with no name that may be of different types.
	 *
	 * Value is a VarUInt byte count, followed by a VarUInt field count, followed by the fields.
	 */
	Array                            = 0x04,
	/**
	 * UniformArray is an array of fields with no name and with the same field type.
	 *
	 * Value is a VarUInt byte count, followed by a VarUInt field count, followed by the field type,
	 * followed by the fields encoded with no type.
	 */
	UniformArray                     = 0x05,

	/** Binary. Value is a VarUInt byte count followed by the data. */
	Binary                           = 0x06,

	/** String in UTF-8. Value is a VarUInt byte count then an unterminated UTF-8 string. */
	String                           = 0x07,

	/**
	 * Non-negative integer with the range of a 64-bit unsigned integer.
	 *
	 * Value is the value encoded as a VarUInt.
	 */
	IntegerPositive                  = 0x08,
	/**
	 * Negative integer with the range of a 64-bit signed integer.
	 *
	 * Value is the ones' complement of the value encoded as a VarUInt.
	 */
	IntegerNegative                  = 0x09,

	/** Single precision float. Value is one big endian IEEE 754 binary32 float. */
	Float32                          = 0x0a,
	/** Double precision float. Value is one big endian IEEE 754 binary64 float. */
	Float64                          = 0x0b,

	/** Boolean false value. Value is empty. */
	BoolFalse                        = 0x0c,
	/** Boolean true value. Value is empty. */
	BoolTrue                         = 0x0d,

	/**
	 * ObjectAttachment is a reference to a compact binary object attachment stored externally.
	 *
	 * Value is a 160-bit hash digest of the referenced compact binary object data.
	 */
	ObjectAttachment                 = 0x0e,
	/**
	 * BinaryAttachment is a reference to a binary attachment stored externally.
	 *
	 * Value is a 160-bit hash digest of the referenced binary data.
	 */
	BinaryAttachment                 = 0x0f,

	/** Hash. Value is a 160-bit hash digest. */
	Hash                             = 0x10,
	/** UUID/GUID. Value is a 128-bit UUID as defined by RFC 4122. */
	Uuid                             = 0x11,

	/**
	 * Date and time between 0001-01-01 00:00:00.0000000 and 9999-12-31 23:59:59.9999999.
	 *
	 * Value is a big endian int64 count of 100ns ticks since 0001-01-01 00:00:00.0000000.
	 */
	DateTime                         = 0x12,
	/**
	 * Difference between two date/time values.
	 *
	 * Value is a big endian int64 count of 100ns ticks in the span, and may be negative.
	 */
	TimeSpan                         = 0x13,

	/**
	 * ObjectId is an opaque object identifier. See FCbObjectId.
	 *
	 * Value is a 12-byte object identifier.
	 */
	ObjectId                         = 0x14,

	/**
	 * CustomById identifies the sub-type of its value by an integer identifier.
	 *
	 * Value is a VarUInt byte count, followed by a VarUInt encoding of the sub-type identifier,
	 * followed by the value of the sub-type.
	 */
	CustomById                       = 0x1e,
	/**
	 * CustomByType identifies the sub-type of its value by a string identifier.
	 *
	 * Value is a VarUInt byte count, followed by a String encoding of the sub-type identifier,
	 * followed by the value of the sub-type.
	 */
	CustomByName                     = 0x1f,

	/** Reserved for future use as a flag. Do not add types in this range. */
	Reserved                         = 0x20,

	/**
	 * A transient flag which indicates that the object or array containing this field has stored
	 * the field type before the value and name. Non-uniform objects and fields will set this.
	 *
	 * Note: Since the flag must never be serialized, this bit may be re-purposed in the future.
	 */
	HasFieldType                     = 0x40,

	/** A persisted flag which indicates that the field has a name stored before the value. */
	HasFieldName                     = 0x80,
};

ENUM_CLASS_FLAGS(ECbFieldType);

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Functions that operate on ECbFieldType. */
class FCbFieldType
{
	static constexpr ECbFieldType SerializedTypeMask    = ECbFieldType(0b1001'1111);
	static constexpr ECbFieldType TypeMask              = ECbFieldType(0b0001'1111);

	static constexpr ECbFieldType ObjectMask            = ECbFieldType(0b0001'1110);
	static constexpr ECbFieldType ObjectBase            = ECbFieldType(0b0000'0010);

	static constexpr ECbFieldType ArrayMask             = ECbFieldType(0b0001'1110);
	static constexpr ECbFieldType ArrayBase             = ECbFieldType(0b0000'0100);

	static constexpr ECbFieldType IntegerMask           = ECbFieldType(0b0001'1110);
	static constexpr ECbFieldType IntegerBase           = ECbFieldType(0b0000'1000);

	static constexpr ECbFieldType FloatMask             = ECbFieldType(0b0001'1100);
	static constexpr ECbFieldType FloatBase             = ECbFieldType(0b0000'1000);

	static constexpr ECbFieldType BoolMask              = ECbFieldType(0b0001'1110);
	static constexpr ECbFieldType BoolBase              = ECbFieldType(0b0000'1100);

	static constexpr ECbFieldType AttachmentMask        = ECbFieldType(0b0001'1110);
	static constexpr ECbFieldType AttachmentBase        = ECbFieldType(0b0000'1110);

	static void StaticAssertTypeConstants();

public:
	/** The type with flags removed. */
	static constexpr inline ECbFieldType GetType(ECbFieldType Type)             { return Type & TypeMask; }
	/** The type with transient flags removed. */
	static constexpr inline ECbFieldType GetSerializedType(ECbFieldType Type)   { return Type & SerializedTypeMask; }

	static constexpr inline bool HasFieldType(ECbFieldType Type) { return EnumHasAnyFlags(Type, ECbFieldType::HasFieldType); }
	static constexpr inline bool HasFieldName(ECbFieldType Type) { return EnumHasAnyFlags(Type, ECbFieldType::HasFieldName); }

	static constexpr inline bool IsNone(ECbFieldType Type)       { return GetType(Type) == ECbFieldType::None; }
	static constexpr inline bool IsNull(ECbFieldType Type)       { return GetType(Type) == ECbFieldType::Null; }

	static constexpr inline bool IsObject(ECbFieldType Type)     { return (Type & ObjectMask) == ObjectBase; }
	static constexpr inline bool IsArray(ECbFieldType Type)      { return (Type & ArrayMask) == ArrayBase; }

	static constexpr inline bool IsBinary(ECbFieldType Type)     { return GetType(Type) == ECbFieldType::Binary; }
	static constexpr inline bool IsString(ECbFieldType Type)     { return GetType(Type) == ECbFieldType::String; }

	static constexpr inline bool IsInteger(ECbFieldType Type)    { return (Type & IntegerMask) == IntegerBase; }
	/** Whether the field is a float, or integer due to implicit conversion. */
	static constexpr inline bool IsFloat(ECbFieldType Type)      { return (Type & FloatMask) == FloatBase; }
	static constexpr inline bool IsBool(ECbFieldType Type)       { return (Type & BoolMask) == BoolBase; }

	static constexpr inline bool IsObjectAttachment(ECbFieldType Type) { return GetType(Type) == ECbFieldType::ObjectAttachment; }
	static constexpr inline bool IsBinaryAttachment(ECbFieldType Type) { return GetType(Type) == ECbFieldType::BinaryAttachment; }
	static constexpr inline bool IsAttachment(ECbFieldType Type)       { return (Type & AttachmentMask) == AttachmentBase; }

	static constexpr inline bool IsHash(ECbFieldType Type)       { return GetType(Type) == ECbFieldType::Hash || IsAttachment(Type); }
	static constexpr inline bool IsUuid(ECbFieldType Type)       { return GetType(Type) == ECbFieldType::Uuid; }

	static constexpr inline bool IsDateTime(ECbFieldType Type)   { return GetType(Type) == ECbFieldType::DateTime; }
	static constexpr inline bool IsTimeSpan(ECbFieldType Type)   { return GetType(Type) == ECbFieldType::TimeSpan; }

	static constexpr inline bool IsObjectId(ECbFieldType Type)   { return GetType(Type) == ECbFieldType::ObjectId; }

	static constexpr inline bool IsCustomById(ECbFieldType Type)   { return GetType(Type) == ECbFieldType::CustomById; }
	static constexpr inline bool IsCustomByName(ECbFieldType Type) { return GetType(Type) == ECbFieldType::CustomByName; }

	static constexpr inline bool HasFields(ECbFieldType Type)
	{
		return GetType(Type) >= ECbFieldType::Object && GetType(Type) <= ECbFieldType::UniformArray;
	}

	static constexpr inline bool HasUniformFields(ECbFieldType Type)
	{
		return GetType(Type) == ECbFieldType::UniformObject || GetType(Type) == ECbFieldType::UniformArray;
	}

	/** Whether the type is or may contain fields of any attachment type. */
	static constexpr inline bool MayContainAttachments(ECbFieldType Type)
	{
		// The use of !! will suppress V792 from static analysis. Using //-V792 did not work.
		return !!IsObject(Type) | !!IsArray(Type) | !!IsAttachment(Type);
	}
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** A reference to a function that is used to visit fields. */
using FCbFieldVisitor = TNotNull<UE::TFunctionWithContext<void (FCbFieldView)>>;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Iterator that can be used as a sentinel for the end of a range. */
class FCbIteratorSentinel
{
};

/**
 * Iterator for FCbField[View] that can operate on any contiguous range of fields.
 *
 * The iterator *is* the current field that the iterator points to and exposes the full interface
 * of FCbField[View]. An iterator that is at the end is equivalent to a field with no value.
 *
 * The iterator represents a range of fields from the current field to the last field.
 */
template <typename FieldType>
class TCbFieldIterator : public FieldType
{
public:
	/** Construct an empty field range. */
	constexpr TCbFieldIterator() = default;

	CORE_API TCbFieldIterator& operator++();

	inline TCbFieldIterator operator++(int)
	{
		TCbFieldIterator It(*this);
		++*this;
		return It;
	}

	constexpr inline FieldType& operator*() { return *this; }
	constexpr inline FieldType* operator->() { return this; }

	/** Reset this to an empty field range. */
	inline void Reset() { *this = TCbFieldIterator(); }

	/** Returns the size of the fields in the range in bytes. */
	CORE_API uint64 GetRangeSize() const;

	/** Calculate the hash of every field in the range. */
	CORE_API FIoHash GetRangeHash() const;
	/** Append the hash of every field in the range. */
	CORE_API void AppendRangeHash(FIoHashBuilder& Builder) const;

	using FieldType::Equals;

	template <typename OtherFieldType>
	constexpr inline bool Equals(const TCbFieldIterator<OtherFieldType>& Other) const
	{
		return FieldType::GetValueData() == Other.OtherFieldType::GetValueData() && FieldsEnd == Other.FieldsEnd;
	}

	template <typename OtherFieldType>
	constexpr inline bool operator==(const TCbFieldIterator<OtherFieldType>& Other) const
	{
		return Equals(Other);
	}

	template <typename OtherFieldType>
	constexpr inline bool operator!=(const TCbFieldIterator<OtherFieldType>& Other) const
	{
		return !Equals(Other);
	}

	constexpr inline bool operator==(const FCbIteratorSentinel&) const { return !FieldType::HasValue(); }
	constexpr inline bool operator!=(const FCbIteratorSentinel&) const { return FieldType::HasValue(); }

	/** Copy the field range into a buffer of exactly GetRangeSize() bytes. */
	CORE_API void CopyRangeTo(FMutableMemoryView Buffer) const;

	/** Copy the field range into an archive, as if calling CopyTo on every field. */
	CORE_API void CopyRangeTo(FArchive& Ar) const;

	/** Invoke the visitor for every attachment in the field range. */
	CORE_API void IterateRangeAttachments(FCbFieldVisitor Visitor) const;

	/**
	 * Try to get a view of every field in the range as they would be serialized.
	 *
	 * A view is available if each field contains its type. Access the equivalent for other field
	 * ranges through FCbFieldIterator::CloneRange or CopyRangeTo.
	 */
	inline bool TryGetRangeView(FMemoryView& OutView) const
	{
		FMemoryView View;
		if (FieldType::TryGetView(View))
		{
			OutView = MakeMemoryView(View.GetData(), FieldsEnd);
			return true;
		}
		return false;
	}

	/** DO NOT USE DIRECTLY. These functions enable range-based for loop support. */
	constexpr inline TCbFieldIterator begin() const { return *this; }
	constexpr inline FCbIteratorSentinel end() const { return FCbIteratorSentinel(); }

protected:
	/** Construct a field range that contains exactly one field. */
	constexpr inline explicit TCbFieldIterator(FieldType InField)
		: FieldType(MoveTemp(InField))
		, FieldsEnd(FieldType::GetValueEnd())
	{
	}

	/**
	 * Construct a field range from the first field and a pointer to the end of the last field.
	 *
	 * @param InField The first field, or the default field if there are no fields.
	 * @param InFieldsEnd A pointer to the end of the value of the last field, or null.
	 */
	constexpr inline TCbFieldIterator(FieldType&& InField, const void* InFieldsEnd)
		: FieldType(MoveTemp(InField))
		, FieldsEnd(InFieldsEnd)
	{
	}

	/** Returns the end of the last field, or null for an iterator at the end. */
	template <typename OtherFieldType>
	static inline const void* GetFieldsEnd(const TCbFieldIterator<OtherFieldType>& It)
	{
		return It.FieldsEnd;
	}

private:
	template <typename OtherType>
	friend class TCbFieldIterator;

	/** Pointer to the first byte past the end of the last field. Set to null at the end. */
	const void* FieldsEnd = nullptr;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Errors that can occur when accessing a field. */
enum class ECbFieldError : uint8
{
	/** The field is not in an error state. */
	None,
	/** The value type does not match the requested type. */
	TypeError,
	/** The value is out of range for the requested type. */
	RangeError,
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * An opaque 12-byte object identifier.
 *
 * It has no intrinsic meaning, and can only be properly interpreted in the context of its usage.
 */
class FCbObjectId
{
public:
	using ByteArray = uint8[12];

	/** Construct an ObjectId with every byte initialized to zero. */
	FCbObjectId() = default;

	/** Construct an ObjectId from an array of 12 bytes. */
	inline explicit FCbObjectId(const ByteArray& ObjectId);

	/** Construct an ObjectId from a view of 12 bytes. */
	CORE_API explicit FCbObjectId(FMemoryView ObjectId);

	/** Returns a reference to the raw byte array for the ObjectId. */
	inline const ByteArray& GetBytes() const { return Bytes; }
	inline operator const ByteArray&() const { return Bytes; }

	/** Returns a view of the raw byte array for the ObjectId. */
	constexpr inline FMemoryView GetView() const { return MakeMemoryView(Bytes); }

	[[nodiscard]] CORE_API static FCbObjectId NewObjectId();

	inline bool operator==(const FCbObjectId& B) const
	{
		return FMemory::Memcmp(this, &B, sizeof(FCbObjectId)) == 0;
	}

	inline bool operator!=(const FCbObjectId& B) const
	{
		return FMemory::Memcmp(this, &B, sizeof(FCbObjectId)) != 0;
	}

	inline bool operator<(const FCbObjectId& B) const
	{
		return FMemory::Memcmp(this, &B, sizeof(FCbObjectId)) <= 0;
	}

	friend inline uint32 GetTypeHash(const FCbObjectId& Id)
	{
		return *reinterpret_cast<const uint32*>(&Id);
	}

	/** Convert the ObjectId to a 24-character hex string. */
	template <typename CharType>
	friend inline TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const FCbObjectId& Id)
	{
		UE::String::BytesToHexLower(Id.GetBytes(), Builder);
		return Builder;
	}

private:
	alignas(uint32) ByteArray Bytes{};
};

inline FCbObjectId::FCbObjectId(const ByteArray& ObjectId)
{
	FMemory::Memcpy(Bytes, ObjectId, sizeof(ByteArray));
}

FGuid			ToGuid(const FCbObjectId& Id);
FCbObjectId		FromGuid(const FGuid& Id);

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** A custom compact binary field type with an integer identifier. */
struct FCbCustomById
{
	/** An identifier for the sub-type of the field. */
	uint64 Id = 0;
	/** A view of the value. Lifetime is tied to the field that the value is associated with. */
	FMemoryView Data;
};

/** A custom compact binary field type with a string identifier. */
struct FCbCustomByName
{
	/** An identifier for the sub-type of the field. Lifetime is tied to the field that the name is associated with. */
	FUtf8StringView Name;
	/** A view of the value. Lifetime is tied to the field that the value is associated with. */
	FMemoryView Data;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::CompactBinary::Private
{

/** Parameters for converting to an integer. */
struct FIntegerParams
{
	/** Whether the output type has a sign bit. */
	uint32 IsSigned : 1;
	/** Bits of magnitude. (7 for int8) */
	uint32 MagnitudeBits : 31;
};

/** Make integer params for the given integer type. */
template <typename IntType>
static constexpr inline FIntegerParams MakeIntegerParams()
{
	FIntegerParams Params;
	Params.IsSigned = IntType(-1) < IntType(0);
	Params.MagnitudeBits = 8 * sizeof(IntType) - Params.IsSigned;
	return Params;
}

} // UE::CompactBinary::Private

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * An atom of data in the compact binary format.
 *
 * Accessing the value of a field is always a safe operation, even if accessed as the wrong type.
 * An invalid access will return a default value for the requested type, and set an error code on
 * the field that can be checked with GetLastError and HasLastError. A valid access will clear an
 * error from a previous invalid access.
 *
 * A field is encoded in one or more bytes, depending on its type and the type of object or array
 * that contains it. A field of an object or array which is non-uniform encodes its field type in
 * the first byte, and includes the HasFieldName flag for a field in an object. The field name is
 * encoded in a variable-length unsigned integer of its size in bytes, for named fields, followed
 * by that many bytes of the UTF-8 encoding of the name with no null terminator. The remainder of
 * the field is the value which is described in the field type enum. Every field must be uniquely
 * addressable when encoded, which means a zero-byte field is not permitted, and only arises in a
 * uniform array of fields with no value, where the answer is to encode as a non-uniform array.
 *
 * This type only provides a view into memory and does not perform any memory management itself.
 * Use FCbField to hold a reference to the underlying memory when necessary.
 */
class FCbFieldView
{
public:
	/** Construct a field with no name and no value. */
	constexpr FCbFieldView() = default;

	/**
	 * Construct a field from a pointer to its data and an optional externally-provided type.
	 *
	 * @param Data Pointer to the start of the field data.
	 * @param Type HasFieldType means that Data contains the type. Otherwise, use the given type.
	 */
	CORE_API explicit FCbFieldView(const void* Data, ECbFieldType Type = ECbFieldType::HasFieldType);

	/** Construct a field from a value, without access to the name. */
	inline explicit FCbFieldView(const FCbValue& Value);

	/** Returns a copy of the field with the name removed. */
	[[nodiscard]] constexpr inline FCbFieldView RemoveName() const
	{
		FCbFieldView Field;
		Field.TypeWithFlags = TypeWithFlags & ~(ECbFieldType::HasFieldType | ECbFieldType::HasFieldName);
		Field.Value = Value;
		return Field;
	}

	/** Returns the name of the field if it has a name, otherwise an empty view. */
	[[nodiscard]] constexpr inline FUtf8StringView GetName() const
	{
		return FUtf8StringView(static_cast<const UTF8CHAR*>(Value) - NameLen, NameLen);
	}

	/** Returns the value for unchecked access. Prefer the typed accessors below. */
	[[nodiscard]] inline FCbValue GetValue() const;

	/** Access the field as an object. Defaults to an empty object on error. */
	[[nodiscard]] CORE_API FCbObjectView AsObjectView();

	/** Access the field as an array. Defaults to an empty array on error. */
	[[nodiscard]] CORE_API FCbArrayView AsArrayView();

	/** Access the field as binary. Returns the provided default on error. */
	[[nodiscard]] CORE_API FMemoryView AsBinaryView(FMemoryView Default = FMemoryView());

	/** Access the field as a string. Returns the provided default on error. */
	[[nodiscard]] CORE_API FUtf8StringView AsString(FUtf8StringView Default = FUtf8StringView());

	/** Access the field as an int8. Returns the provided default on error. */
	[[nodiscard]] inline int8 AsInt8(int8 Default = 0) { return AsInteger<int8>(Default); }
	/** Access the field as an int16. Returns the provided default on error. */
	[[nodiscard]] inline int16 AsInt16(int16 Default = 0) { return AsInteger<int16>(Default); }
	/** Access the field as an int32. Returns the provided default on error. */
	[[nodiscard]] inline int32 AsInt32(int32 Default = 0) { return AsInteger<int32>(Default); }
	/** Access the field as an int64. Returns the provided default on error. */
	[[nodiscard]] inline int64 AsInt64(int64 Default = 0) { return AsInteger<int64>(Default); }
	/** Access the field as a uint8. Returns the provided default on error. */
	[[nodiscard]] inline uint8 AsUInt8(uint8 Default = 0) { return AsInteger<uint8>(Default); }
	/** Access the field as a uint16. Returns the provided default on error. */
	[[nodiscard]] inline uint16 AsUInt16(uint16 Default = 0) { return AsInteger<uint16>(Default); }
	/** Access the field as a uint32. Returns the provided default on error. */
	[[nodiscard]] inline uint32 AsUInt32(uint32 Default = 0) { return AsInteger<uint32>(Default); }
	/** Access the field as a uint64. Returns the provided default on error. */
	[[nodiscard]] inline uint64 AsUInt64(uint64 Default = 0) { return AsInteger<uint64>(Default); }

	/** Access the field as a float. Returns the provided default on error. */
	[[nodiscard]] CORE_API float AsFloat(float Default = 0.0f);
	/** Access the field as a double. Returns the provided default on error. */
	[[nodiscard]] CORE_API double AsDouble(double Default = 0.0);

	/** Access the field as a bool. Returns the provided default on error. */
	[[nodiscard]] CORE_API bool AsBool(bool bDefault = false);

	/** Access the field as a hash referencing an object attachment. Returns the provided default on error. */
	[[nodiscard]] CORE_API FIoHash AsObjectAttachment(const FIoHash& Default = FIoHash());
	/** Access the field as a hash referencing a binary attachment. Returns the provided default on error. */
	[[nodiscard]] CORE_API FIoHash AsBinaryAttachment(const FIoHash& Default = FIoHash());
	/** Access the field as a hash referencing an attachment. Returns the provided default on error. */
	[[nodiscard]] CORE_API FIoHash AsAttachment(const FIoHash& Default = FIoHash());

	/** Access the field as a hash. Returns the provided default on error. */
	[[nodiscard]] CORE_API FIoHash AsHash(const FIoHash& Default = FIoHash());

	/** Access the field as a UUID. Returns a nil UUID on error. */
	[[nodiscard]] CORE_API FGuid AsUuid();
	/** Access the field as a UUID. Returns the provided default on error. */
	[[nodiscard]] CORE_API FGuid AsUuid(const FGuid& Default);

	/** Access the field as a date/time tick count. Returns the provided default on error. */
	[[nodiscard]] CORE_API int64 AsDateTimeTicks(int64 Default = 0);

	/** Access the field as a date/time. Returns a date/time at the epoch on error. */
	[[nodiscard]] CORE_API FDateTime AsDateTime();
	/** Access the field as a date/time. Returns the provided default on error. */
	[[nodiscard]] CORE_API FDateTime AsDateTime(FDateTime Default);

	/** Access the field as a timespan tick count. Returns the provided default on error. */
	[[nodiscard]] CORE_API int64 AsTimeSpanTicks(int64 Default = 0);

	/** Access the field as a timespan. Returns an empty timespan on error. */
	[[nodiscard]] CORE_API FTimespan AsTimeSpan();
	/** Access the field as a timespan. Returns the provided default on error. */
	[[nodiscard]] CORE_API FTimespan AsTimeSpan(FTimespan Default);

	/** Access the field as an object identifier. Returns the provided default on error. */
	[[nodiscard]] CORE_API FCbObjectId AsObjectId(const FCbObjectId& Default = FCbObjectId());

	/** Access the field as a custom sub-type with an integer identifier. Returns the provided default on error. */
	[[nodiscard]] CORE_API FCbCustomById AsCustomById(FCbCustomById Default = FCbCustomById());
	/** Access the field as a custom sub-type with a string identifier. Returns the provided default on error. */
	[[nodiscard]] CORE_API FCbCustomByName AsCustomByName(FCbCustomByName Default = FCbCustomByName());

	/** Access the field as a custom sub-type with an integer identifier. Returns the provided default on error. */
	[[nodiscard]] CORE_API FMemoryView AsCustom(uint64 Id, FMemoryView Default = FMemoryView());
	/** Access the field as a custom sub-type with a string identifier. Returns the provided default on error. */
	[[nodiscard]] CORE_API FMemoryView AsCustom(FUtf8StringView Name, FMemoryView Default = FMemoryView());

	/** True if the field has a name. */
	constexpr inline bool HasName() const           { return FCbFieldType::HasFieldName(TypeWithFlags); }

	constexpr inline bool IsNull() const            { return FCbFieldType::IsNull(TypeWithFlags); }

	constexpr inline bool IsObject() const          { return FCbFieldType::IsObject(TypeWithFlags); }
	constexpr inline bool IsArray() const           { return FCbFieldType::IsArray(TypeWithFlags); }

	constexpr inline bool IsBinary() const          { return FCbFieldType::IsBinary(TypeWithFlags); }
	constexpr inline bool IsString() const          { return FCbFieldType::IsString(TypeWithFlags); }

	/** Whether the field is an integer of unspecified range and sign. */
	constexpr inline bool IsInteger() const         { return FCbFieldType::IsInteger(TypeWithFlags); }
	/** Whether the field is a float, or integer that supports implicit conversion. */
	constexpr inline bool IsFloat() const           { return FCbFieldType::IsFloat(TypeWithFlags); }
	constexpr inline bool IsBool() const            { return FCbFieldType::IsBool(TypeWithFlags); }

	constexpr inline bool IsObjectAttachment() const { return FCbFieldType::IsObjectAttachment(TypeWithFlags); }
	constexpr inline bool IsBinaryAttachment() const { return FCbFieldType::IsBinaryAttachment(TypeWithFlags); }
	constexpr inline bool IsAttachment() const       { return FCbFieldType::IsAttachment(TypeWithFlags); }

	constexpr inline bool IsHash() const            { return FCbFieldType::IsHash(TypeWithFlags); }
	constexpr inline bool IsUuid() const            { return FCbFieldType::IsUuid(TypeWithFlags); }

	constexpr inline bool IsDateTime() const        { return FCbFieldType::IsDateTime(TypeWithFlags); }
	constexpr inline bool IsTimeSpan() const        { return FCbFieldType::IsTimeSpan(TypeWithFlags); }

	constexpr inline bool IsObjectId() const        { return FCbFieldType::IsObjectId(TypeWithFlags); }

	constexpr inline bool IsCustomById() const      { return FCbFieldType::IsCustomById(TypeWithFlags); }
	constexpr inline bool IsCustomByName() const    { return FCbFieldType::IsCustomByName(TypeWithFlags); }

	/** Whether the field has a value. */
	constexpr inline explicit operator bool() const { return HasValue(); }

	/**
	 * Whether the field has a value.
	 *
	 * All fields in a valid object or array have a value. A field with no value is returned when
	 * finding a field by name fails or when accessing an iterator past the end.
	 */
	constexpr inline bool HasValue() const          { return !FCbFieldType::IsNone(TypeWithFlags); };

	/** Whether the last field access encountered an error. */
	constexpr inline bool HasError() const          { return Error != ECbFieldError::None; }

	/** The type of error that occurred on the last field access, or None. */
	[[nodiscard]] constexpr inline ECbFieldError GetError() const { return Error; }

	/** Returns the size of the field in bytes, including the type and name. */
	[[nodiscard]] CORE_API uint64 GetSize() const;

	/** Calculate the hash of the field, including the type and name. */
	[[nodiscard]] CORE_API FIoHash GetHash() const;
	/** Append the hash of the field, including the type and name. */
	CORE_API void AppendHash(FIoHashBuilder& Builder) const;

	/**
	 * Whether this field is identical to the other field.
	 *
	 * Performs a deep comparison of any contained arrays or objects and their fields. Comparison
	 * assumes that both fields are valid and are written in the canonical format. Fields must be
	 * written in the same order in arrays and objects, and name comparison is case sensitive. If
	 * these assumptions do not hold, this may return false for equivalent inputs. Validation can
	 * be performed with ValidateCompactBinary, except for field order and field name case.
	 */
	CORE_API bool Equals(const FCbFieldView& Other) const;

	/** Copy the field into a buffer of exactly GetSize() bytes, including the type and name. */
	CORE_API void CopyTo(FMutableMemoryView Buffer) const;

	/** Copy the field into an archive, including its type and name. */
	CORE_API void CopyTo(FArchive& Ar) const;

	/** Invoke the visitor for every attachment in the field. */
	CORE_API void IterateAttachments(FCbFieldVisitor Visitor) const;

	/**
	 * Try to get a view of the field as it would be serialized, such as by CopyTo.
	 *
	 * A view is available if the field contains its type. Access the equivalent for other fields
	 * through FCbField::GetBuffer, FCbField::Clone, or CopyTo.
	 */
	inline bool TryGetView(FMemoryView& OutView) const
	{
		if (FCbFieldType::HasFieldType(TypeWithFlags))
		{
			OutView = GetView();
			return true;
		}
		return false;
	}

	/** Find a field of an object by case-sensitive name comparison, otherwise a field with no value. */
	[[nodiscard]] CORE_API FCbFieldView operator[](FUtf8StringView Name) const;

	/** Create an iterator for the fields of an array or object, otherwise an empty iterator. */
	[[nodiscard]] CORE_API FCbFieldViewIterator CreateViewIterator() const;

	/** DO NOT USE DIRECTLY. These functions enable range-based for loop support. */
	inline FCbFieldViewIterator begin() const;
	constexpr inline FCbIteratorSentinel end() const { return FCbIteratorSentinel(); }

protected:
	/** Returns a view of the field, including the type and name when present. */
	CORE_API FMemoryView GetView() const;

	/** Returns a view of the name and value, which excludes the type. */
	CORE_API FMemoryView GetViewNoType() const;

	/** Returns a view of the value, which excludes the type and name. */
	inline FMemoryView GetValueView() const { return MakeMemoryView(Value, GetValueSize()); }

	/** Returns the type of the field excluding flags. */
	constexpr inline ECbFieldType GetType() const { return FCbFieldType::GetType(TypeWithFlags); }

	/** Returns the type of the field including flags. */
	constexpr inline ECbFieldType GetTypeWithFlags() const { return TypeWithFlags; }

	/** Returns the start of the value. */
	constexpr inline const void* GetValueData() const { return Value; }

	/** Returns the end of the value. */
	inline const void* GetValueEnd() const { return static_cast<const uint8*>(Value) + GetValueSize(); }

	/** Returns the size of the value in bytes, which is the field excluding the type and name. */
	CORE_API uint64 GetValueSize() const;

	/** Assign a field from a pointer to its data and an optional externally-provided type. */
	inline void Assign(const void* InData, const ECbFieldType InType)
	{
		static_assert(std::is_trivially_destructible_v<FCbFieldView>,
			"This optimization requires FCbFieldView to be trivially destructible!");
		new(this) FCbFieldView(InData, InType);
	}

private:
	/**
	 * Access the field as the given integer type.
	 *
	 * Returns the provided default if the value cannot be represented in the output type.
	 */
	template <typename IntType>
	inline IntType AsInteger(IntType Default)
	{
		return IntType(AsInteger(uint64(Default), UE::CompactBinary::Private::MakeIntegerParams<IntType>()));
	}

	CORE_API uint64 AsInteger(uint64 Default, UE::CompactBinary::Private::FIntegerParams Params);

private:
	/** The field type, with the transient HasFieldType flag if the field contains its type. */
	ECbFieldType TypeWithFlags = ECbFieldType::None;
	/** The error (if any) that occurred on the last field access. */
	ECbFieldError Error = ECbFieldError::None;
	/** The number of bytes for the name stored before the value. */
	uint32 NameLen = 0;
	/** The value, which also points to the end of the name. */
	const void* Value = nullptr;
};

/**
 * Iterator for FCbFieldView.
 *
 * @see TCbFieldIterator
 */
class FCbFieldViewIterator : public TCbFieldIterator<FCbFieldView>
{
public:
	/** Construct a field range that contains exactly one field. */
	[[nodiscard]] static inline FCbFieldViewIterator MakeSingle(const FCbFieldView& Field)
	{
		return FCbFieldViewIterator(Field);
	}

	/**
	 * Construct a field range from a buffer containing zero or more valid fields.
	 *
	 * @param View A buffer containing zero or more valid fields.
	 * @param Type HasFieldType means that View contains the type. Otherwise, use the given type.
	 */
	[[nodiscard]] static inline FCbFieldViewIterator MakeRange(FMemoryView View, ECbFieldType Type = ECbFieldType::HasFieldType)
	{
		return !View.IsEmpty() ? FCbFieldViewIterator(FCbFieldView(View.GetData(), Type), View.GetDataEnd()) : FCbFieldViewIterator();
	}

	/** Construct an empty field range. */
	constexpr FCbFieldViewIterator() = default;

	/** Construct an iterator from another iterator. */
	template <typename OtherFieldType>
	inline FCbFieldViewIterator(const TCbFieldIterator<OtherFieldType>& It)
		: TCbFieldIterator(ImplicitConv<FCbFieldView>(It), GetFieldsEnd(It))
	{
	}

private:
	using TCbFieldIterator::TCbFieldIterator;
};

inline FCbFieldViewIterator FCbFieldView::begin() const
{
	return CreateViewIterator();
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Array of FCbField[View] that have no names.
 *
 * Accessing a field of the array requires iteration. Access by index is not provided because the
 * cost of accessing an item by index scales linearly with the index.
 *
 * This type only provides a view into memory and does not perform any memory management itself.
 * Use FCbArray to hold a reference to the underlying memory when necessary.
 */
class FCbArrayView : protected FCbFieldView
{
public:
	/** @see FCbFieldView::FCbFieldView */
	using FCbFieldView::FCbFieldView;

	/** Construct an array with no fields. */
	CORE_API FCbArrayView();

	/** Returns the number of items in the array. */
	CORE_API uint64 Num() const;

	/** Access the array as an array field. */
	[[nodiscard]] inline FCbFieldView AsFieldView() const { return RemoveName(); }

	/** Construct an array from an array field. No type check is performed! */
	[[nodiscard]] static inline FCbArrayView FromFieldNoCheck(const FCbFieldView& Field) { return FCbArrayView(Field); }

	/** Whether the array has any fields. */
	inline explicit operator bool() const { return Num() > 0; }

	/** Returns the size of the array in bytes if serialized by itself with no name. */
	CORE_API uint64 GetSize() const;

	/** Calculate the hash of the array if serialized by itself with no name. */
	CORE_API FIoHash GetHash() const;
	/** Append the hash of the array if serialized by itself with no name. */
	CORE_API void AppendHash(FIoHashBuilder& Builder) const;

	/**
	 * Whether this array is identical to the other array.
	 *
	 * Performs a deep comparison of any contained arrays or objects and their fields. Comparison
	 * assumes that both fields are valid and are written in the canonical format. Fields must be
	 * written in the same order in arrays and objects, and name comparison is case sensitive. If
	 * these assumptions do not hold, this may return false for equivalent inputs. Validation can
	 * be done with the All mode to check these assumptions about the format of the inputs.
	 */
	CORE_API bool Equals(const FCbArrayView& Other) const;

	/** Copy the array into a buffer of exactly GetSize() bytes, with no name. */
	CORE_API void CopyTo(FMutableMemoryView Buffer) const;

	/** Copy the array into an archive. This will write GetSize() bytes, with no name. */
	CORE_API void CopyTo(FArchive& Ar) const;

	/** Invoke the visitor for every attachment in the array. */
	inline void IterateAttachments(FCbFieldVisitor Visitor) const { CreateViewIterator().IterateRangeAttachments(Visitor); }

	/**
	 * Try to get a view of the array as it would be serialized, such as by CopyTo.
	 *
	 * A view is available if the array contains its type and has no name. Access the equivalent
	 * for other arrays through FCbArray::GetBuffer, FCbArray::Clone, or CopyTo.
	 */
	inline bool TryGetView(FMemoryView& OutView) const
	{
		return !FCbFieldView::HasName() && FCbFieldView::TryGetView(OutView);
	}

	/** @see FCbFieldView::CreateViewIterator */
	using FCbFieldView::CreateViewIterator;
	using FCbFieldView::begin;
	using FCbFieldView::end;

private:
	/** Construct an array from an array field. No type check is performed! Use via FromFieldNoCheck. */
	inline explicit FCbArrayView(const FCbFieldView& Field) : FCbFieldView(Field) {}
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Array of FCbField[View] that have unique names.
 *
 * Accessing the fields of an object is always a safe operation, even if the requested field does
 * not exist. Fields may be accessed by name or through iteration. When a field is requested that
 * is not found in the object, the field that it returns has no value (evaluates to false) though
 * attempting to access the empty field is also safe, as described by FCbFieldView.
 *
 * This type only provides a view into memory and does not perform any memory management itself.
 * Use FCbObject to hold a reference to the underlying memory when necessary.
 */
class FCbObjectView : protected FCbFieldView
{
public:
	/** @see FCbFieldView::FCbFieldView */
	using FCbFieldView::FCbFieldView;

	/** Construct an object with no fields. */
	CORE_API FCbObjectView();

	/**
	 * Find a field by case-sensitive name comparison.
	 *
	 * The cost of this operation scales linearly with the number of fields in the object. Prefer to
	 * iterate over the fields only once when consuming an object.
	 *
	 * @param Name The name of the field.
	 * @return The matching field if found, otherwise a field with no value.
	 */
	[[nodiscard]] CORE_API FCbFieldView FindView(FUtf8StringView Name) const;

	/** Find a field by case-insensitive name comparison. */
	[[nodiscard]] CORE_API FCbFieldView FindViewIgnoreCase(FUtf8StringView Name) const;

	/** Find a field by case-sensitive name comparison. */
	[[nodiscard]] inline FCbFieldView operator[](FUtf8StringView Name) const { return FindView(Name); }

	/** Access the object as an object field. */
	[[nodiscard]] inline FCbFieldView AsFieldView() const { return RemoveName(); }

	/** Construct an object from an object field. No type check is performed! */
	[[nodiscard]] static inline FCbObjectView FromFieldNoCheck(const FCbFieldView& Field) { return FCbObjectView(Field); }

	/** Whether the object has any fields. */
	CORE_API explicit operator bool() const;

	/** Returns the size of the object in bytes if serialized by itself with no name. */
	CORE_API uint64 GetSize() const;

	/** Calculate the hash of the object if serialized by itself with no name. */
	CORE_API FIoHash GetHash() const;
	/** Append the hash of the object if serialized by itself with no name. */
	CORE_API void AppendHash(FIoHashBuilder& Builder) const;

	/**
	 * Whether this object is identical to the other object.
	 *
	 * Performs a deep comparison of any contained arrays or objects and their fields. Comparison
	 * assumes that both fields are valid and are written in the canonical format. Fields must be
	 * written in the same order in arrays and objects, and name comparison is case sensitive. If
	 * these assumptions do not hold, this may return false for equivalent inputs. Validation can
	 * be done with the All mode to check these assumptions about the format of the inputs.
	 */
	CORE_API bool Equals(const FCbObjectView& Other) const;

	/** Copy the object into a buffer of exactly GetSize() bytes, with no name. */
	CORE_API void CopyTo(FMutableMemoryView Buffer) const;

	/** Copy the object into an archive. This will write GetSize() bytes, with no name. */
	CORE_API void CopyTo(FArchive& Ar) const;

	/** Invoke the visitor for every attachment in the object. */
	inline void IterateAttachments(FCbFieldVisitor Visitor) const { CreateViewIterator().IterateRangeAttachments(Visitor); }

	/**
	 * Try to get a view of the object as it would be serialized, such as by CopyTo.
	 *
	 * A view is available if the object contains its type and has no name. Access the equivalent
	 * for other objects through FCbObject::GetBuffer, FCbObject::Clone, or CopyTo.
	 */
	inline bool TryGetView(FMemoryView& OutView) const
	{
		return !FCbFieldView::HasName() && FCbFieldView::TryGetView(OutView);
	}

	/** @see FCbFieldView::CreateViewIterator */
	using FCbFieldView::CreateViewIterator;
	using FCbFieldView::begin;
	using FCbFieldView::end;

private:
	/** Construct an object from an object field. No type check is performed! Use via FromFieldNoCheck. */
	inline explicit FCbObjectView(const FCbFieldView& Field) : FCbFieldView(Field) {}
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** A wrapper that holds a reference to the buffer that contains its compact binary value. */
template <typename ViewType>
class TCbBuffer : public ViewType
{
public:
	/** Construct a default value. */
	TCbBuffer() = default;

	/**
	 * Construct a value from a pointer to its data and an optional externally-provided type.
	 *
	 * @param ValueBuffer A buffer that exactly contains the value.
	 * @param Type HasFieldType means that ValueBuffer contains the type. Otherwise, use the given type.
	 */
	inline explicit TCbBuffer(FSharedBuffer ValueBuffer, ECbFieldType Type = ECbFieldType::HasFieldType)
	{
		if (ValueBuffer)
		{
			ViewType::operator=(ViewType(ValueBuffer.GetData(), Type));
			check(ValueBuffer.GetView().Contains(ViewType::GetView()));
			Buffer = MoveTemp(ValueBuffer);
		}
	}

	/** Construct a value that holds a reference to the buffer that contains it. */
	inline TCbBuffer(const ViewType& Value, FSharedBuffer OuterBuffer)
		: ViewType(Value)
	{
		if (OuterBuffer)
		{
			check(OuterBuffer.GetView().Contains(ViewType::GetView()));
			Buffer = MoveTemp(OuterBuffer);
		}
	}

	/** Construct a value that holds a reference to the buffer of the outer that contains it. */
	template <typename OtherViewType>
	inline TCbBuffer(const ViewType& Value, TCbBuffer<OtherViewType> OuterBuffer)
		: TCbBuffer(Value, MoveTemp(OuterBuffer.Buffer))
	{
	}

	/** Reset this to a default value and null buffer. */
	inline void Reset() { *this = TCbBuffer(); }

	/** Whether this reference has ownership of the memory in its buffer. */
	inline bool IsOwned() const { return Buffer && Buffer.IsOwned(); }

	/** Clone the value, if necessary, to a buffer that this reference has ownership of. */
	inline void MakeOwned()
	{
		if (!IsOwned())
		{
			FUniqueBuffer MutableBuffer = FUniqueBuffer::Alloc(ViewType::GetSize());
			ViewType::CopyTo(MutableBuffer);
			ViewType::operator=(ViewType(MutableBuffer.GetData()));
			Buffer = MutableBuffer.MoveToShared();
		}
	}

	/** Returns the value as a view. */
	[[nodiscard]] inline const ViewType& AsView() const { return *this; }

	/**
	 * Returns the outer buffer (if any) that contains this value.
	 *
	 * The outer buffer might contain other data before and/or after this value. Use GetBuffer to
	 * request a buffer that exactly contains this value, or TryGetView for a contiguous view.
	 */
	[[nodiscard]] inline const FSharedBuffer& GetOuterBuffer() const & { return Buffer; }
	[[nodiscard]] inline FSharedBuffer GetOuterBuffer() && { return MoveTemp(Buffer); }

	/** Find a field of an object by case-sensitive name comparison, otherwise a field with no value. */
	[[nodiscard]] inline FCbField operator[](FUtf8StringView Name) const;

	/** Create an iterator for the fields of an array or object, otherwise an empty iterator. */
	[[nodiscard]] inline FCbFieldIterator CreateIterator() const;

	/** DO NOT USE DIRECTLY. These functions enable range-based for loop support. */
	inline FCbFieldIterator begin() const;
	constexpr inline FCbIteratorSentinel end() const { return FCbIteratorSentinel(); }

private:
	template <typename OtherType>
	friend class TCbBuffer;

	FSharedBuffer Buffer;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Factory functions for types derived from TCbBuffer.
 *
 * This uses the curiously recurring template pattern to construct the correct derived type, that
 * must inherit from TCbBuffer and this type to expose the factory functions.
 */
template <typename Type, typename ViewType>
class TCbBufferFactory
{
public:
	/** Construct a value from an owned clone of its memory. */
	[[nodiscard]] static inline Type Clone(const void* const Data)
	{
		return Clone(ViewType(Data));
	}

	/** Construct a value from an owned clone of its memory. */
	[[nodiscard]] static inline Type Clone(const ViewType& Value)
	{
		Type Owned = MakeView(Value);
		Owned.MakeOwned();
		return Owned;
	}

	/** Construct a value from a read-only view of its memory and its optional outer buffer. */
	[[nodiscard]] static inline Type MakeView(const void* const Data, FSharedBuffer OuterBuffer = FSharedBuffer())
	{
		return MakeView(ViewType(Data), MoveTemp(OuterBuffer));
	}

	/** Construct a value from a read-only view of its memory and its optional outer buffer. */
	[[nodiscard]] static inline Type MakeView(const ViewType& Value, FSharedBuffer OuterBuffer = FSharedBuffer())
	{
		return Type(Value, MoveTemp(OuterBuffer));
	}
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FCbArray;
class FCbObject;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * A field that includes a shared buffer for the memory that contains it.
 *
 * @see FCbFieldView
 * @see TCbBuffer
 */
class FCbField : public TCbBuffer<FCbFieldView>, public TCbBufferFactory<FCbField, FCbFieldView>
{
public:
	using TCbBuffer::TCbBuffer;

	/** Access the field as an object. Defaults to an empty object on error. */
	[[nodiscard]] inline FCbObject AsObject() &;
	[[nodiscard]] inline FCbObject AsObject() &&;

	/** Access the field as an array. Defaults to an empty array on error. */
	[[nodiscard]] inline FCbArray AsArray()&;
	[[nodiscard]] inline FCbArray AsArray() &&;

	/** Access the field as binary. Returns the provided default on error. */
	[[nodiscard]] inline FSharedBuffer AsBinary(const FSharedBuffer& Default = FSharedBuffer())&;
	[[nodiscard]] inline FSharedBuffer AsBinary(const FSharedBuffer& Default = FSharedBuffer()) &&;

	/** Returns a buffer that contains the field as it would be serialized by CopyTo. */
	[[nodiscard]] CORE_API FCompositeBuffer GetBuffer() const;
};

template <typename ViewType>
inline FCbField TCbBuffer<ViewType>::operator[](FUtf8StringView Name) const
{
	if (FCbFieldView Field = ViewType::operator[](Name))
	{
		return FCbField::MakeView(Field, GetOuterBuffer());
	}
	return FCbField();
}

/**
 * Iterator for FCbField.
 *
 * @see TCbFieldIterator
 */
class FCbFieldIterator : public TCbFieldIterator<FCbField>
{
public:
	/** Construct a field range from an owned clone of a range. */
	[[nodiscard]] CORE_API static FCbFieldIterator CloneRange(const FCbFieldViewIterator& It);

	/** Construct a field range from an owned clone of a range. */
	[[nodiscard]] static inline FCbFieldIterator CloneRange(const FCbFieldIterator& It)
	{
		return CloneRange(FCbFieldViewIterator(It));
	}

	/** Construct a field range that contains exactly one field. */
	[[nodiscard]] static inline FCbFieldIterator MakeSingle(FCbField Field)
	{
		return FCbFieldIterator(MoveTemp(Field));
	}

	/**
	 * Construct a field range from a buffer containing zero or more valid fields.
	 *
	 * @param Buffer A buffer containing zero or more valid fields.
	 * @param Type HasFieldType means that Buffer contains the type. Otherwise, use the given type.
	 */
	[[nodiscard]] static inline FCbFieldIterator MakeRange(FSharedBuffer Buffer, ECbFieldType Type = ECbFieldType::HasFieldType)
	{
		if (Buffer.GetSize())
		{
			const void* const DataEnd = Buffer.GetView().GetDataEnd();
			return FCbFieldIterator(FCbField(MoveTemp(Buffer), Type), DataEnd);
		}
		return FCbFieldIterator();
	}

	/** Construct a field range from an iterator and its optional outer buffer. */
	[[nodiscard]] static inline FCbFieldIterator MakeRangeView(const FCbFieldViewIterator& It, FSharedBuffer OuterBuffer = FSharedBuffer())
	{
		return FCbFieldIterator(FCbField(It, MoveTemp(OuterBuffer)), GetFieldsEnd(It));
	}

	/** Construct an empty field range. */
	constexpr FCbFieldIterator() = default;

	/** Clone the range, if necessary, to a buffer that this has ownership of. */
	inline void MakeRangeOwned()
	{
		if (!IsOwned())
		{
			*this = CloneRange(*this);
		}
	}

	/** Call MakeRangeOwned() to make the range of the iterator owned. */
	void MakeOwned() = delete;

private:
	using TCbFieldIterator::TCbFieldIterator;
};

template <typename ViewType>
inline FCbFieldIterator TCbBuffer<ViewType>::CreateIterator() const
{
	if (FCbFieldViewIterator It = ViewType::CreateViewIterator())
	{
		return FCbFieldIterator::MakeRangeView(It, GetOuterBuffer());
	}
	return FCbFieldIterator();
}

template <typename ViewType>
inline FCbFieldIterator TCbBuffer<ViewType>::begin() const
{
	return CreateIterator();
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * An array that includes a shared buffer for the memory that contains it.
 *
 * @see FCbArrayView
 * @see TCbBuffer
 */
class FCbArray : public TCbBuffer<FCbArrayView>, public TCbBufferFactory<FCbArray, FCbArrayView>
{
public:
	using TCbBuffer::TCbBuffer;

	/** Access the array as an array field. */
	[[nodiscard]] inline FCbField AsField() const & { return FCbField(FCbArrayView::AsFieldView(), *this); }
	[[nodiscard]] inline FCbField AsField() && { return FCbField(FCbArrayView::AsFieldView(), MoveTemp(*this)); }

	/** Returns a buffer that contains the array as it would be serialized by CopyTo. */
	[[nodiscard]] CORE_API FCompositeBuffer GetBuffer() const;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * An object that includes a shared buffer for the memory that contains it.
 *
 * @see FCbObjectView
 * @see TCbBuffer
 */
class FCbObject : public TCbBuffer<FCbObjectView>, public TCbBufferFactory<FCbObject, FCbObjectView>
{
public:
	using TCbBuffer::TCbBuffer;

	/** Find a field by case-sensitive name comparison. */
	[[nodiscard]] inline FCbField Find(FUtf8StringView Name) const
	{
		if (::FCbFieldView Field = FindView(Name))
		{
			return FCbField(Field, *this);
		}
		return FCbField();
	}

	/** Find a field by case-insensitive name comparison. */
	[[nodiscard]] inline FCbField FindIgnoreCase(FUtf8StringView Name) const
	{
		if (::FCbFieldView Field = FindViewIgnoreCase(Name))
		{
			return FCbField(Field, *this);
		}
		return FCbField();
	}

	/** Find a field by case-sensitive name comparison. */
	[[nodiscard]] inline FCbField operator[](FUtf8StringView Name) const { return Find(Name); }

	/** Access the object as an object field. */
	[[nodiscard]] inline FCbField AsField() const & { return FCbField(FCbObjectView::AsFieldView(), *this); }
	[[nodiscard]] inline FCbField AsField() && { return FCbField(FCbObjectView::AsFieldView(), MoveTemp(*this)); }

	/** Returns a buffer that contains the object as it would be serialized by CopyTo. */
	[[nodiscard]] CORE_API FCompositeBuffer GetBuffer() const;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

inline FCbObject FCbField::AsObject() &
{
	const FCbObjectView View = AsObjectView();
	return !HasError() ? FCbObject(View, *this) : FCbObject();
}

inline FCbObject FCbField::AsObject() &&
{
	const FCbObjectView View = AsObjectView();
	return !HasError() ? FCbObject(View, MoveTemp(*this)) : FCbObject();
}

inline FCbArray FCbField::AsArray() &
{
	const FCbArrayView View = AsArrayView();
	return !HasError() ? FCbArray(View, *this) : FCbArray();
}

inline FCbArray FCbField::AsArray() &&
{
	const FCbArrayView View = AsArrayView();
	return !HasError() ? FCbArray(View, MoveTemp(*this)) : FCbArray();
}

inline FSharedBuffer FCbField::AsBinary(const FSharedBuffer& Default) &
{
	const FMemoryView View = AsBinaryView();
	return !HasError() ? FSharedBuffer::MakeView(View, GetOuterBuffer()) : Default;
}

inline FSharedBuffer FCbField::AsBinary(const FSharedBuffer& Default) &&
{
	const FMemoryView View = AsBinaryView();
	return !HasError() ? FSharedBuffer::MakeView(View, MoveTemp(*this).GetOuterBuffer()) : Default;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

// Types to support debug visualizers.

// A field/array/object in memory can be visualized in a debugger supporting natvis by casting to this type.
// (FDebugCb*)Pointer works for any field/array/object that starts with its field type.
struct UE_INTERNAL FDebugCb {};

// Types that visualize a pointer to a value, *not* a pointer to a field.
struct UE_INTERNAL FDebugCbObject {};
struct UE_INTERNAL FDebugCbUniformObject {};
struct UE_INTERNAL FDebugCbArray {};
struct UE_INTERNAL FDebugCbUniformArray {};
struct UE_INTERNAL FDebugCbBinary {};
struct UE_INTERNAL FDebugCbString {};
struct UE_INTERNAL FDebugCbIntegerPositive {};
struct UE_INTERNAL FDebugCbIntegerNegative {};
struct UE_INTERNAL FDebugCbFloat32 {};
struct UE_INTERNAL FDebugCbFloat64 {};
struct UE_INTERNAL FDebugCbObjectAttachment {};
struct UE_INTERNAL FDebugCbBinaryAttachment {};
struct UE_INTERNAL FDebugCbHash {};
struct UE_INTERNAL FDebugCbUuid {};
struct UE_INTERNAL FDebugCbDateTime {};
struct UE_INTERNAL FDebugCbTimeSpan {};
struct UE_INTERNAL FDebugCbObjectId {};
struct UE_INTERNAL FDebugCbCustomById {};
struct UE_INTERNAL FDebugCbCustomByName {};

namespace UE::CompactBinary::Private
{

PRAGMA_DISABLE_INTERNAL_WARNINGS;
CORE_API void ReferenceDebugTypes(
	FDebugCb*,
	FDebugCbObject*,
	FDebugCbUniformObject*,
	FDebugCbArray*,
	FDebugCbUniformArray*,
	FDebugCbBinary*,
	FDebugCbString*,
	FDebugCbIntegerPositive*,
	FDebugCbIntegerNegative*,
	FDebugCbFloat32*,
	FDebugCbFloat64*,
	FDebugCbObjectAttachment*,
	FDebugCbBinaryAttachment*,
	FDebugCbHash*,
	FDebugCbUuid*,
	FDebugCbDateTime*,
	FDebugCbTimeSpan*,
	FDebugCbObjectId*,
	FDebugCbCustomById*,
	FDebugCbCustomByName*);
PRAGMA_ENABLE_INTERNAL_WARNINGS;

} // UE::CompactBinary::Private

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Memory/CompositeBuffer.h"
#include "Templates/Function.h"
#include "Templates/RemoveReference.h"
#endif
