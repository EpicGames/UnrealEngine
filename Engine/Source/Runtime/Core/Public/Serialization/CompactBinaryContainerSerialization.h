// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Templates/Tuple.h"

template <typename... Types>
inline FCbWriter& operator<<(FCbWriter& Writer, const TTuple<Types...>& Tuple)
{
	Writer.BeginArray();
	VisitTupleElements([&Writer](auto&& ElementValue) { Writer << ElementValue; }, Tuple);
	Writer.EndArray();
	return Writer;
}

template <typename... Types>
inline bool LoadFromCompactBinary(FCbFieldView Field, TTuple<Types...>& Tuple)
{
	bool bOk = Field.AsArrayView().Num() == sizeof...(Types);
	FCbFieldViewIterator ElementIterator = Field.CreateViewIterator();

	VisitTupleElements([&ElementIterator, &bOk](auto&& ElementValue)
		{
			bOk = LoadFromCompactBinary(ElementIterator++, ElementValue) & bOk;
		},
		Tuple
	);
	return bOk;
}

