// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compression/CompressedBuffer.h"
#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "CoreTypes.h"
#include "IO/IoHash.h"
#include "Memory/CompositeBuffer.h"
#include "Memory/SharedBuffer.h"
#include "Misc/AssertionMacros.h"
#include "Misc/TVariant.h"
#include "Serialization/CompactBinary.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

class FArchive;
class FCbWriter;
template <typename FuncType> class TFunctionRef;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * An attachment is either null, raw binary, compressed binary, or an object.
 *
 * Attachments are always identified by their raw hash, even when stored compressed.
 *
 * An attachment is serialized as a sequence of compact binary fields with no name. It is invalid
 * to serialize a null attachment. A raw binary attachment is serialized as an empty Binary field
 * when empty, and otherwise as a BinaryAttachment field containing the raw hash, followed by the
 * raw binary data in a Binary field. A compressed binary attachment is serialized as Binary with
 * a compressed buffer as the value. An object is serialized as an empty Object field when empty,
 * and otherwise as an ObjectAttachment field containing the hash, followed by the object data in
 * an Object field.
 */
class FCbAttachment
{
	struct FObjectValue
	{
		FCbObject Object;
		FIoHash Hash;

		CORE_API FObjectValue(FCbObject&& Object, const FIoHash* Hash);
	};

	struct FBinaryValue
	{
		FCompositeBuffer Buffer;
		FIoHash Hash;

		template <typename BufferType, decltype(FCompositeBuffer(DeclVal<BufferType>().MakeOwned()))* = nullptr>
		inline explicit FBinaryValue(BufferType&& InBuffer)
			: Buffer(Forward<BufferType>(InBuffer).MakeOwned())
			, Hash(FIoHash::HashBuffer(Buffer))
		{
		}

		template <typename BufferType, decltype(FCompositeBuffer(DeclVal<BufferType>().MakeOwned()))* = nullptr>
		inline explicit FBinaryValue(BufferType&& InBuffer, const FIoHash& InHash)
			: Buffer(Forward<BufferType>(InBuffer).MakeOwned())
			, Hash(InHash)
		{
			checkSlow(Hash == FIoHash::HashBuffer(Buffer));
		}
	};

	struct FCompressedBinaryValue
	{
		FCompressedBuffer Buffer;
		FIoHash Hash;

		template <typename BufferType, decltype(FCompressedBuffer(DeclVal<BufferType>().MakeOwned()))* = nullptr>
		inline explicit FCompressedBinaryValue(BufferType&& InBuffer)
			: Buffer(Forward<BufferType>(InBuffer).MakeOwned())
			, Hash(Buffer.GetRawHash())
		{
		}

		template <typename BufferType, decltype(FCompressedBuffer(DeclVal<BufferType>().MakeOwned()))* = nullptr>
		inline explicit FCompressedBinaryValue(BufferType&& InBuffer, const FIoHash& InHash)
			: Buffer(Forward<BufferType>(InBuffer).MakeOwned())
			, Hash(InHash)
		{
		}
	};

	template <typename... ArgTypes, decltype(FBinaryValue(DeclVal<ArgTypes>()...))* = nullptr>
	inline FCbAttachment(TInPlaceType<FBinaryValue>, ArgTypes&&... Args)
		: Value(TInPlaceType<FBinaryValue>(), Forward<ArgTypes>(Args)...)
	{
		if (Value.Get<FBinaryValue>().Buffer.IsNull())
		{
			Value.Emplace<TYPE_OF_NULLPTR>();
		}
	}

public:
	/** Construct a null attachment. */
	FCbAttachment() = default;

	/** Construct an object attachment. Value is cloned if not owned. */
	inline explicit FCbAttachment(const FCbObject& InValue)
		: Value(TInPlaceType<FObjectValue>(), FCbObject(InValue), nullptr) {}
	inline explicit FCbAttachment(FCbObject&& InValue)
		: Value(TInPlaceType<FObjectValue>(), MoveTemp(InValue), nullptr) {}
	inline explicit FCbAttachment(const FCbObject& InValue, const FIoHash& Hash)
		: Value(TInPlaceType<FObjectValue>(), FCbObject(InValue), &Hash) {}
	inline explicit FCbAttachment(FCbObject&& InValue, const FIoHash& Hash)
		: Value(TInPlaceType<FObjectValue>(), MoveTemp(InValue), &Hash) {}

	/** Construct a raw binary attachment from a shared/composite buffer. Value is cloned if not owned. */
	template <typename BufferType, decltype(FBinaryValue(DeclVal<BufferType>().MakeOwned()))* = nullptr>
	inline explicit FCbAttachment(BufferType&& InValue)
		: FCbAttachment(TInPlaceType<FBinaryValue>(), Forward<BufferType>(InValue).MakeOwned()) {}
	template <typename BufferType, decltype(FBinaryValue(DeclVal<BufferType>().MakeOwned()))* = nullptr>
	inline explicit FCbAttachment(BufferType&& InValue, const FIoHash& InHash)
		: FCbAttachment(TInPlaceType<FBinaryValue>(), Forward<BufferType>(InValue).MakeOwned(), InHash) {}

	/** Construct a compressed binary attachment. Value is cloned if not owned. */
	inline explicit FCbAttachment(const FCompressedBuffer& InValue)
		: Value(TInPlaceType<FCompressedBinaryValue>(), InValue)
	{
		if (Value.Get<FCompressedBinaryValue>().Buffer.IsNull())
		{
			Value.Emplace<TYPE_OF_NULLPTR>();
		}
	}

	UE_INTERNAL inline explicit FCbAttachment(const FCompressedBuffer& InValue, const FIoHash& Hash)
		: Value(TInPlaceType<FCompressedBinaryValue>(), InValue, Hash)
	{
		if (Value.Get<FCompressedBinaryValue>().Buffer.IsNull())
		{
			Value.Emplace<TYPE_OF_NULLPTR>();
		}
	}

	inline explicit FCbAttachment(FCompressedBuffer&& InValue)
		: Value(TInPlaceType<FCompressedBinaryValue>(), InValue)
	{
		if (Value.Get<FCompressedBinaryValue>().Buffer.IsNull())
		{
			Value.Emplace<TYPE_OF_NULLPTR>();
		}
	}

	UE_INTERNAL inline explicit FCbAttachment(FCompressedBuffer&& InValue, const FIoHash& Hash)
		: Value(TInPlaceType<FCompressedBinaryValue>(), InValue, Hash)
	{
		if (Value.Get<FCompressedBinaryValue>().Buffer.IsNull())
		{
			Value.Emplace<TYPE_OF_NULLPTR>();
		}
	}

	/** Reset this to a null attachment. */
	inline void Reset() { *this = FCbAttachment(); }

	/** Access the attachment as an object. Defaults to an empty object on error. */
	inline FCbObject AsObject() const;

	/** Access the attachment as raw binary in a single contiguous buffer. Defaults to a null buffer on error. */
	inline FSharedBuffer AsBinary() const;

	/** Access the attachment as raw binary. Defaults to a null buffer on error. */
	inline const FCompositeBuffer& AsCompositeBinary() const;

	/** Access the attachment as compressed binary. Defaults to a null buffer on error. */
	inline const FCompressedBuffer& AsCompressedBinary() const;

	/** Whether the attachment has a non-null value. */
	inline explicit operator bool() const { return !IsNull(); }

	/** Whether the attachment has a null value. */
	inline bool IsNull() const { return Value.IsType<TYPE_OF_NULLPTR>(); }

	/** Returns whether the attachment is an object. */
	inline bool IsObject() const { return Value.IsType<FObjectValue>(); }

	/** Returns whether the attachment is raw binary. */
	inline bool IsBinary() const { return Value.IsType<FBinaryValue>(); }

	/** Returns whether the attachment is compressed binary. */
	inline bool IsCompressedBinary() const { return Value.IsType<FCompressedBinaryValue>(); }

	/** Returns the hash of the attachment value. */
	CORE_API FIoHash GetHash() const;

	/** Compares attachments by their hash. Any discrepancy in type must be handled externally. */
	inline bool operator==(const FCbAttachment& Attachment) const { return GetHash() == Attachment.GetHash(); }
	inline bool operator!=(const FCbAttachment& Attachment) const { return GetHash() != Attachment.GetHash(); }
	inline bool operator<(const FCbAttachment& Attachment) const { return GetHash() < Attachment.GetHash(); }

	/**
	 * Load the attachment from compact binary as written by Save.
	 *
	 * The attachment references the input iterator if it is owned, and otherwise clones the value.
	 *
	 * The iterator is advanced as attachment fields are consumed from it.
	 */
	CORE_API bool TryLoad(FCbFieldIterator& Fields);

	/**
	 * Load the attachment from compact binary as written by Save.
	 *
	 * The attachments value will be loaded into an owned buffer.
	 *
	 * @param Ar Archive to read the attachment from. An error state is set on failure.
	 * @param Allocator Allocator for the attachment value buffer.
	 * @note Allocated buffers will be cloned if they are not owned.
	 */
	CORE_API bool TryLoad(FArchive& Ar, FCbBufferAllocator Allocator = FUniqueBuffer::Alloc);

	/** Save the attachment into the writer as a stream of compact binary fields. */
	CORE_API void Save(FCbWriter& Writer) const;

	/** Save the attachment into the archive as a stream of compact binary fields. */
	CORE_API void Save(FArchive& Ar) const;

private:
	TVariant<TYPE_OF_NULLPTR, FObjectValue, FBinaryValue, FCompressedBinaryValue> Value;
};

/** Hashes attachments by their hash. Any discrepancy in type must be handled externally. */
inline uint32 GetTypeHash(const FCbAttachment& Attachment)
{
	return GetTypeHash(Attachment.GetHash());
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * A package is an object with a tree of referenced attachments.
 *
 * A package is a Merkle tree with an object as its root and non-leaf nodes, and either binary or
 * objects as its leaf nodes. Nodes reference their children from fields of type BinaryAttachment
 * or ObjectAttachment, which store the raw hash of the referenced attachment.
 *
 * It is invalid for a package to include attachments that are not referenced by its object or by
 * one of its referenced object attachments. This invariant needs to be maintained if attachments
 * are added explicitly instead of being discovered by the attachment resolver. If any attachment
 * is not referenced, it may not survive a round-trip through certain storage systems.
 *
 * It is valid for a package to exclude referenced attachments, but then it is the responsibility
 * of the package consumer to have a mechanism for resolving those references when necessary.
 *
 * A package is serialized as a sequence of compact binary fields with no name. The object may be
 * both preceded and followed by attachments. The object is only serialized when it is non-empty,
 * starting with its hash, in a Hash field, followed by the object, in an Object field. A package
 * ends with a Null field. The canonical order of components is the object and its hash, followed
 * by the attachments ordered by hash, followed by a Null field. It is valid for the a package to
 * have its attachments serialized in any order relative to each other and to the object.
 */
class FCbPackage
{
public:
	/**
	 * A function that resolves a hash to a buffer containing the data matching that hash.
	 *
	 * The resolver may return a null buffer to skip resolving an attachment for the hash.
	 */
	using FAttachmentResolver = TFunctionRef<FSharedBuffer (const FIoHash& Hash)>;

	/** Construct a null package. */
	FCbPackage() = default;

	/**
	 * Construct a package from a root object without gathering attachments.
	 *
	 * @param InObject The root object, which will be cloned unless it is owned.
	 */
	inline explicit FCbPackage(FCbObject InObject)
	{
		SetObject(MoveTemp(InObject));
	}

	/**
	 * Construct a package from a root object and gather attachments using the resolver.
	 *
	 * @param InObject The root object, which will be cloned unless it is owned.
	 * @param InResolver A function that is invoked for every reference and binary reference field.
	 */
	inline explicit FCbPackage(FCbObject InObject, FAttachmentResolver InResolver)
	{
		SetObject(MoveTemp(InObject), InResolver);
	}

	/**
	 * Construct a package from a root object without gathering attachments.
	 *
	 * @param InObject The root object, which will be cloned unless it is owned.
	 * @param InObjectHash The hash of the object, which must match to avoid validation errors.
	 */
	inline explicit FCbPackage(FCbObject InObject, const FIoHash& InObjectHash)
	{
		SetObject(MoveTemp(InObject), InObjectHash);
	}

	/**
	 * Construct a package from a root object and gather attachments using the resolver.
	 *
	 * @param InObject The root object, which will be cloned unless it is owned.
	 * @param InObjectHash The hash of the object, which must match to avoid validation errors.
	 * @param InResolver A function that is invoked for every reference and binary reference field.
	 */
	inline explicit FCbPackage(FCbObject InObject, const FIoHash& InObjectHash, FAttachmentResolver InResolver)
	{
		SetObject(MoveTemp(InObject), InObjectHash, InResolver);
	}

	/** Reset this to a null package. */
	inline void Reset() { *this = FCbPackage(); }

	/** Whether the package has a non-empty object or attachments. */
	inline explicit operator bool() const { return !IsNull(); }

	/** Whether the package has an empty object and no attachments. */
	inline bool IsNull() const { return !Object && Attachments.IsEmpty(); }

	/** Returns the object for the package. */
	inline const FCbObject& GetObject() const { return Object; }

	/** Returns the hash of the object for the package. */
	inline const FIoHash& GetObjectHash() const { return ObjectHash; }

	/**
	 * Set the root object without gathering attachments.
	 *
	 * @param InObject The root object, which will be cloned unless it is owned.
	 */
	inline void SetObject(FCbObject InObject)
	{
		SetObject(MoveTemp(InObject), nullptr, nullptr);
	}

	/**
	 * Set the root object and gather attachments using the resolver.
	 *
	 * @param InObject The root object, which will be cloned unless it is owned.
	 * @param InResolver A function that is invoked for every reference and binary reference field.
	 */
	inline void SetObject(FCbObject InObject, FAttachmentResolver InResolver)
	{
		SetObject(MoveTemp(InObject), nullptr, &InResolver);
	}

	/**
	 * Set the root object without gathering attachments.
	 *
	 * @param InObject The root object, which will be cloned unless it is owned.
	 * @param InObjectHash The hash of the object, which must match to avoid validation errors.
	 */
	inline void SetObject(FCbObject InObject, const FIoHash& InObjectHash)
	{
		SetObject(MoveTemp(InObject), &InObjectHash, nullptr);
	}

	/**
	 * Set the root object and gather attachments using the resolver.
	 *
	 * @param InObject The root object, which will be cloned unless it is owned.
	 * @param InObjectHash The hash of the object, which must match to avoid validation errors.
	 * @param InResolver A function that is invoked for every reference and binary reference field.
	 */
	inline void SetObject(FCbObject InObject, const FIoHash& InObjectHash, FAttachmentResolver InResolver)
	{
		SetObject(MoveTemp(InObject), &InObjectHash, &InResolver);
	}

	/** Returns the attachments in this package. */
	inline TConstArrayView<FCbAttachment> GetAttachments() const { return Attachments; }

	/**
	 * Find an attachment by its hash.
	 *
	 * @return The attachment, or null if the attachment is not found.
	 * @note The returned pointer is only valid until the attachments on this package are modified.
	 */
	CORE_API const FCbAttachment* FindAttachment(const FIoHash& Hash) const;

	/** Find an attachment if it exists in the package. */
	inline const FCbAttachment* FindAttachment(const FCbAttachment& Attachment) const
	{
		return FindAttachment(Attachment.GetHash());
	}

	/** Add the attachment to this package. */
	inline void AddAttachment(const FCbAttachment& Attachment)
	{
		AddAttachment(Attachment, nullptr);
	}

	/** Add the attachment to this package, along with any references that can be resolved. */
	inline void AddAttachment(const FCbAttachment& Attachment, FAttachmentResolver Resolver)
	{
		AddAttachment(Attachment, &Resolver);
	}

	/**
	 * Remove an attachment by hash.
	 *
	 * @return Number of attachments removed, which will be either 0 or 1.
	 */
	CORE_API int32 RemoveAttachment(const FIoHash& Hash);
	inline int32 RemoveAttachment(const FCbAttachment& Attachment) { return RemoveAttachment(Attachment.GetHash()); }

	/** Compares packages by their object and attachment hashes. */
	CORE_API bool Equals(const FCbPackage& Package) const;
	inline bool operator==(const FCbPackage& Package) const { return Equals(Package); }
	inline bool operator!=(const FCbPackage& Package) const { return !Equals(Package); }

	/**
	 * Load the object and attachments from compact binary as written by Save.
	 *
	 * The object and attachments reference the input iterator, if it is owned, and otherwise clones
	 * the object and attachments individually to make owned copies.
	 *
	 * The iterator is advanced as object and attachment fields are consumed from it.
	 */
	CORE_API bool TryLoad(FCbFieldIterator& Fields);

	/**
	 * Load the object and attachments from compact binary as written by Save.
	 *
	 * The object and attachments will be individually loaded into owned buffers.
	 *
	 * @param Ar Archive to read the package from. An error state is set on failure.
	 * @param Allocator Allocator for object and attachment buffers.
	 * @note Allocated buffers will be cloned if they are not owned.
	 */
	CORE_API bool TryLoad(FArchive& Ar, FCbBufferAllocator Allocator = FUniqueBuffer::Alloc);

	/** Save the object and attachments into the writer as a stream of compact binary fields. */
	CORE_API void Save(FCbWriter& Writer) const;

	/** Save the object and attachments into the archive as a stream of compact binary fields. */
	CORE_API void Save(FArchive& Ar) const;

private:
	CORE_API void SetObject(FCbObject Object, const FIoHash* Hash, FAttachmentResolver* Resolver);
	CORE_API void AddAttachment(const FCbAttachment& Attachment, FAttachmentResolver* Resolver);

	void GatherAttachments(const FCbObject& Object, FAttachmentResolver Resolver);

	/** Attachments ordered by their hash. */
	TArray<FCbAttachment> Attachments;
	FCbObject Object;
	FIoHash ObjectHash;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCbObject FCbAttachment::AsObject() const
{
	if (const FObjectValue* ObjectValue = Value.TryGet<FObjectValue>())
	{
		return ObjectValue->Object;
	}
	return FCbObject();
}

FSharedBuffer FCbAttachment::AsBinary() const
{
	if (const FBinaryValue* BinaryValue = Value.TryGet<FBinaryValue>())
	{
		return BinaryValue->Buffer.ToShared();
	}
	return FSharedBuffer();
}

const FCompositeBuffer& FCbAttachment::AsCompositeBinary() const
{
	if (const FBinaryValue* BinaryValue = Value.TryGet<FBinaryValue>())
	{
		return BinaryValue->Buffer;
	}
	return FCompositeBuffer::Null;
}

const FCompressedBuffer& FCbAttachment::AsCompressedBinary() const
{
	if (const FCompressedBinaryValue* CompressedBuffer = Value.TryGet<FCompressedBinaryValue>())
	{
		return CompressedBuffer->Buffer;
	}
	return FCompressedBuffer::Null;
}
