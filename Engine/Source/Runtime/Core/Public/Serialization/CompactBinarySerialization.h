// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "CoreTypes.h"
#include "IO/IoHash.h"
#include "Memory/MemoryFwd.h"
#include "Memory/MemoryView.h"
#include "Memory/SharedBuffer.h"
#include "Serialization/CompactBinary.h"
#include "Templates/Function.h"

class FArchive;
class FName;
struct FDateTime;
struct FGuid;
enum class ECbValidateMode : uint32;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Serialization::Private
{

/** Utility for logging problems with FCbField. For internal use only */
CORE_API void LogFieldTooLargeForArrayWarning(uint64 ArraySize);

} // namespace UE::Serialization::Private

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Determine the size in bytes of the compact binary field at the start of the view.
 *
 * This may be called on an incomplete or invalid field, in which case the returned size is zero.
 * A size can always be extracted from a valid field with no name if a view of at least the first
 * 10 bytes is provided, regardless of field size. For fields with names, the size of view needed
 * to calculate a size is at most 10 + MaxNameLen + MeasureVarUInt(MaxNameLen).
 *
 * This function can be used when streaming a field, for example, to determine the size of buffer
 * to fill before attempting to construct a field from it.
 *
 * @param View A memory view that may contain the start of a field.
 * @param Type HasFieldType means that View contains the type. Otherwise, use the given type.
 */
CORE_API uint64 MeasureCompactBinary(FMemoryView View, ECbFieldType Type = ECbFieldType::HasFieldType);

/**
 * Try to determine the type and size of the compact binary field at the start of the view.
 *
 * This may be called on an incomplete or invalid field, in which case it will return false, with
 * OutSize being 0 for invalid fields, otherwise the minimum view size necessary to make progress
 * in measuring the field on the next call to this function.
 *
 * @note A return of true from this function does not indicate that the entire field is valid.
 *
 * @param InView A memory view that may contain the start of a field.
 * @param OutType The type (with flags) of the field. None is written until a value is available.
 * @param OutSize The total field size for a return of true, 0 for invalid fields, or the size to
 *                make progress in measuring the field on the next call to this function.
 * @param InType HasFieldType means that InView contains the type. Otherwise, use the given type.
 * @return true if the size of the field was determined, otherwise false.
 */
CORE_API bool TryMeasureCompactBinary(
	FMemoryView InView,
	ECbFieldType& OutType,
	uint64& OutSize,
	ECbFieldType InType = ECbFieldType::HasFieldType);

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Load a compact binary field from an archive.
 *
 * The field may be an array or an object, which the caller can convert to by using AsArray or
 * AsObject as appropriate. The buffer allocator is called to provide the buffer for the field
 * to load into once its size has been determined. Validated with ECbValidateMode::Default.
 *
 * @param Ar Archive to read the field from. An error state is set on failure.
 * @param Allocator Allocator for the buffer that the field is loaded into.
 * @return A field with a reference to the allocated buffer, or a default field on failure.
 */
CORE_API FCbField LoadCompactBinary(FArchive& Ar, FCbBufferAllocator Allocator = FUniqueBuffer::Alloc);
/** Load a compact binary field view from an archive. Allocator must manage lifetime. */
CORE_API FCbFieldView LoadCompactBinaryView(FArchive& Ar, FCbMemoryAllocator Allocator);

/**
 * Load a compact binary field from an archive.
 *
 * The field may be an array or an object, which the caller can convert to by using AsArray or
 * AsObject as appropriate. The buffer allocator is called to provide the buffer for the field
 * to load into once its size has been determined.
 *
 * @param Ar Archive to read the field from. An error state is set on failure.
 * @param Allocator Allocator for the buffer that the field is loaded into. May be FUniqueBuffer::Alloc.
 * @param ValidateMode Validation mode in which to validate the loaded field. May be None.
 * @return A field with a reference to the allocated buffer, or a default field on failure.
 */
CORE_API FCbField LoadAndValidateCompactBinary(FArchive& Ar, FCbBufferAllocator Allocator, ECbValidateMode ValidateMode);
/** Load a compact binary field view from an archive. Allocator must manage lifetime. */
CORE_API FCbFieldView LoadAndValidateCompactBinaryView(FArchive& Ar, FCbMemoryAllocator Allocator, ECbValidateMode ValidateMode);

/** Save a compact binary value to an archive. */
CORE_API void SaveCompactBinary(FArchive& Ar, const FCbFieldView& Field);
CORE_API void SaveCompactBinary(FArchive& Ar, const FCbArrayView& Array);
CORE_API void SaveCompactBinary(FArchive& Ar, const FCbObjectView& Object);

/** Serialize a compact binary value to/from an archive. */
CORE_API FArchive& operator<<(FArchive& Ar, FCbField& Field);
CORE_API FArchive& operator<<(FArchive& Ar, FCbArray& Array);
CORE_API FArchive& operator<<(FArchive& Ar, FCbObject& Object);

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * LoadFromCompactBinary attempts to load the output value from compact binary.
 *
 * Implementations of LoadCompactBinary are expected to assign the output even on failure.
 * Implementations may accept an optional default value to assign in case of load failure.
 *
 * @return true if required fields had the values with the correct type and range, otherwise false.
 */

CORE_API bool LoadFromCompactBinary(FCbFieldView Field, FUtf8StringBuilderBase& OutValue);
CORE_API bool LoadFromCompactBinary(FCbFieldView Field, FWideStringBuilderBase& OutValue);
CORE_API bool LoadFromCompactBinary(FCbFieldView Field, UE::FUtf8SharedString& OutString);
CORE_API bool LoadFromCompactBinary(FCbFieldView Field, UE::FWideSharedString& OutString);
CORE_API bool LoadFromCompactBinary(FCbFieldView Field, FUtf8String& OutValue);
CORE_API bool LoadFromCompactBinary(FCbFieldView Field, FWideString& OutValue);
CORE_API bool LoadFromCompactBinary(FCbFieldView Field, FName& OutValue);

inline bool LoadFromCompactBinary(FCbFieldView Field, int8& OutValue, const int8 Default = 0)
{
	OutValue = Field.AsInt8(Default);
	return !Field.HasError();
}

inline bool LoadFromCompactBinary(FCbFieldView Field, int16& OutValue, const int16 Default = 0)
{
	OutValue = Field.AsInt16(Default);
	return !Field.HasError();
}

inline bool LoadFromCompactBinary(FCbFieldView Field, int32& OutValue, const int32 Default = 0)
{
	OutValue = Field.AsInt32(Default);
	return !Field.HasError();
}

inline bool LoadFromCompactBinary(FCbFieldView Field, int64& OutValue, const int64 Default = 0)
{
	OutValue = Field.AsInt64(Default);
	return !Field.HasError();
}

inline bool LoadFromCompactBinary(FCbFieldView Field, uint8& OutValue, const uint8 Default = 0)
{
	OutValue = Field.AsUInt8(Default);
	return !Field.HasError();
}

inline bool LoadFromCompactBinary(FCbFieldView Field, uint16& OutValue, const uint16 Default = 0)
{
	OutValue = Field.AsUInt16(Default);
	return !Field.HasError();
}

inline bool LoadFromCompactBinary(FCbFieldView Field, uint32& OutValue, const uint32 Default = 0)
{
	OutValue = Field.AsUInt32(Default);
	return !Field.HasError();
}

inline bool LoadFromCompactBinary(FCbFieldView Field, uint64& OutValue, const uint64 Default = 0)
{
	OutValue = Field.AsUInt64(Default);
	return !Field.HasError();
}

inline bool LoadFromCompactBinary(FCbFieldView Field, float& OutValue, const float Default = 0.0f)
{
	OutValue = Field.AsFloat(Default);
	return !Field.HasError();
}

inline bool LoadFromCompactBinary(FCbFieldView Field, double& OutValue, const double Default = 0.0)
{
	OutValue = Field.AsDouble(Default);
	return !Field.HasError();
}

inline bool LoadFromCompactBinary(FCbFieldView Field, bool& OutValue, const bool Default = false)
{
	OutValue = Field.AsBool(Default);
	return !Field.HasError();
}

inline bool LoadFromCompactBinary(FCbFieldView Field, FIoHash& OutValue, const FIoHash& Default = FIoHash())
{
	OutValue = Field.AsHash(Default);
	return !Field.HasError();
}

CORE_API bool LoadFromCompactBinary(FCbFieldView Field, FGuid& OutValue);
CORE_API bool LoadFromCompactBinary(FCbFieldView Field, FGuid& OutValue, const FGuid& Default);

CORE_API bool LoadFromCompactBinary(FCbFieldView Field, FDateTime& OutValue);
CORE_API bool LoadFromCompactBinary(FCbFieldView Field, FDateTime& OutValue, FDateTime Default);

inline bool LoadFromCompactBinary(FCbFieldView Field, FCbObjectId& OutValue, const FCbObjectId& Default = FCbObjectId())
{
	OutValue = Field.AsObjectId(Default);
	return !Field.HasError();
}

template <typename T, typename Allocator>
inline bool LoadFromCompactBinary(FCbFieldView Field, TArray<T, Allocator>& OutValue)
{
	const uint64 Size = Field.AsArrayView().Num();
	if (Size <= MAX_int32)
	{
		OutValue.Reset((int32)Size);
		bool bOk = !Field.HasError();
		for (const FCbFieldView& ElementField : Field)
		{
			bOk &= LoadFromCompactBinary(ElementField, OutValue.Emplace_GetRef());
		}
		return bOk;
	}
	else
	{
		UE::Serialization::Private::LogFieldTooLargeForArrayWarning(Size);
		return false;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Convert the compact binary to JSON in a multi-line format with indentation. */
CORE_API void CompactBinaryToJson(const FCbFieldView& Field, FUtf8StringBuilderBase& Builder);
CORE_API void CompactBinaryToJson(const FCbFieldView& Field, FWideStringBuilderBase& Builder);
CORE_API void CompactBinaryToJson(const FCbArrayView& Array, FUtf8StringBuilderBase& Builder);
CORE_API void CompactBinaryToJson(const FCbArrayView& Array, FWideStringBuilderBase& Builder);
CORE_API void CompactBinaryToJson(const FCbObjectView& Object, FUtf8StringBuilderBase& Builder);
CORE_API void CompactBinaryToJson(const FCbObjectView& Object, FWideStringBuilderBase& Builder);

/** Convert the compact binary to JSON in a compact format with no added whitespace. */
CORE_API void CompactBinaryToCompactJson(const FCbFieldView& Field, FUtf8StringBuilderBase& Builder);
CORE_API void CompactBinaryToCompactJson(const FCbFieldView& Field, FWideStringBuilderBase& Builder);
CORE_API void CompactBinaryToCompactJson(const FCbArrayView& Array, FUtf8StringBuilderBase& Builder);
CORE_API void CompactBinaryToCompactJson(const FCbArrayView& Array, FWideStringBuilderBase& Builder);
CORE_API void CompactBinaryToCompactJson(const FCbObjectView& Object, FUtf8StringBuilderBase& Builder);
CORE_API void CompactBinaryToCompactJson(const FCbObjectView& Object, FWideStringBuilderBase& Builder);

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
