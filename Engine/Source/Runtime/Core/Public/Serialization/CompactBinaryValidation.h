// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Memory/MemoryFwd.h"
#include "Memory/MemoryView.h"
#include "Misc/EnumClassFlags.h"
#include "Serialization/CompactBinary.h"

class FCbAttachment;
class FCbPackage;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Flags for validating compact binary data. */
enum class ECbValidateMode : uint32
{
	/** Skip validation if no other validation modes are enabled. */
	None                    = 0,

	/**
	 * Validate that the value can be read and stays inside the bounds of the memory view.
	 *
	 * This is the minimum level of validation required to be able to safely read a field, array,
	 * or object without the risk of crashing or reading out of bounds.
	 */
	Default                 = 1 << 0,

	/**
	 * Validate that object fields have unique non-empty names and array fields have no names.
	 *
	 * Name validation failures typically do not inhibit reading the input, but duplicated fields
	 * cannot be looked up by name other than the first, and converting to other data formats can
	 * fail in the presence of naming issues.
	 */
	Names                   = 1 << 1,

	/**
	 * Validate that fields are serialized in the canonical format.
	 *
	 * Format validation failures typically do not inhibit reading the input. Values that fail in
	 * this mode require more memory than in the canonical format, and comparisons of such values
	 * for equality are not reliable. Examples of failures include uniform arrays or objects that
	 * were not encoded uniformly, variable-length integers that could be encoded in fewer bytes,
	 * or 64-bit floats that could be encoded in 32 bits without loss of precision.
	 */
	Format                  = 1 << 2,

	/**
	 * Validate that there is no padding after the value before the end of the memory view.
	 *
	 * Padding validation failures have no impact on the ability to read the input, but are using
	 * more memory than necessary.
	 */
	Padding                 = 1 << 3,

	/**
	 * Validate that a package or attachment has the expected fields.
	 */
	Package                 = 1 << 4,

	/**
	 * Validate that a package or attachment matches its saved hashes.
	 */
	PackageHash             = 1 << 5,

	/** Perform all validation described above. */
	All                     = Default | Names | Format | Padding | Package | PackageHash,
};

ENUM_CLASS_FLAGS(ECbValidateMode);

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Flags for compact binary validation errors. Multiple flags may be combined. */
enum class ECbValidateError : uint32
{
	/** The input had no validation errors. */
	None                    = 0,

	// Mode: Default

	/** The input cannot be read without reading out of bounds. */
	OutOfBounds             = 1 << 0,
	/** The input has a field with an unrecognized or invalid type. */
	InvalidType             = 1 << 1,

	// Mode: Names

	/** An object had more than one field with the same name. */
	DuplicateName           = 1 << 2,
	/** An object had a field with no name. */
	MissingName             = 1 << 3,
	/** An array field had a name. */
	ArrayName               = 1 << 4,

	// Mode: Format

	/** A name or string value is not valid UTF-8. */
	InvalidString           = 1 << 5,
	/** A size or integer value can be encoded in fewer bytes. */
	InvalidInteger          = 1 << 6,
	/** A float64 value can be encoded as a float32 without loss of precision. */
	InvalidFloat            = 1 << 7,
	/** An object has the same type for every field but is not uniform. */
	NonUniformObject        = 1 << 8,
	/** An array has the same type for every field and non-empty values but is not uniform. */
	NonUniformArray         = 1 << 9,

	// Mode: Padding

	/** A value did not use the entire memory view given for validation. */
	Padding                 = 1 << 10,

	// Mode: Package

	/** The package or attachment had missing fields or fields out of order. */
	InvalidPackageFormat    = 1 << 11,
	/** The object or an attachment did not match the hash stored for it. */
	InvalidPackageHash      = 1 << 12,
	/** The package contained more than one copy of the same attachment. */
	DuplicateAttachments    = 1 << 13,
	/** The package contained more than one object. */
	MultiplePackageObjects  = 1 << 14,
	/** The package contained an object with no fields. */
	NullPackageObject       = 1 << 15,
	/** The package contained a null attachment. */
	NullPackageAttachment   = 1 << 16,
};

ENUM_CLASS_FLAGS(ECbValidateError);

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Validate the compact binary data for one field in the view as specified by the mode flags.
 *
 * Only one top-level field is processed from the view, and validation recurses into any array or
 * object within that field. To validate multiple consecutive top-level fields, call the function
 * once for each top-level field. If the given view might contain multiple top-level fields, then
 * either exclude the Padding flag from the Mode or use MeasureCompactBinary to break up the view
 * into its constituent fields before validating.
 *
 * @param View A memory view containing at least one top-level field.
 * @param Mode A combination of the flags for the types of validation to perform.
 * @param Type HasFieldType means that View contains the type. Otherwise, use the given type.
 * @return None on success, otherwise the flags for the types of errors that were detected.
 */
CORE_API ECbValidateError ValidateCompactBinary(FMemoryView View, ECbValidateMode Mode, ECbFieldType Type = ECbFieldType::HasFieldType);

/**
 * Validate the compact binary data for every field in the view as specified by the mode flags.
 *
 * This function expects the entire view to contain fields. Any trailing region of the view which
 * does not contain a valid field will produce an OutOfBounds or InvalidType error instead of the
 * Padding error that would be produced by the single field validation function.
 *
 * @see ValidateCompactBinary
 */
CORE_API ECbValidateError ValidateCompactBinaryRange(FMemoryView View, ECbValidateMode Mode);

/**
 * Validate the compact binary attachment pointed to by the view as specified by the mode flags.
 *
 * The attachment is validated with ValidateCompactBinary by using the validation mode specified.
 * Include ECbValidateMode::Package to validate the attachment format and hash.
 *
 * @see ValidateCompactBinary
 *
 * @param View A memory view containing a package.
 * @param Mode A combination of the flags for the types of validation to perform.
 * @return None on success, otherwise the flags for the types of errors that were detected.
 */
CORE_API ECbValidateError ValidateCompactBinaryAttachment(FMemoryView View, ECbValidateMode Mode);

/**
 * Validate the compact binary package pointed to by the view as specified by the mode flags.
 *
 * The package, and attachments, are validated with ValidateCompactBinary by using the validation
 * mode specified. Include ECbValidateMode::Package to validate the package format and hashes.
 *
 * @see ValidateCompactBinary
 *
 * @param View A memory view containing a package.
 * @param Mode A combination of the flags for the types of validation to perform.
 * @return None on success, otherwise the flags for the types of errors that were detected.
 */
CORE_API ECbValidateError ValidateCompactBinaryPackage(FMemoryView View, ECbValidateMode Mode);

/**
 * Validate the compact binary data for one value as specified by the mode flags.
 *
 * Validation recurses into attachments, objects, arrays, and fields within the top-level value.
 *
 * @return None on success, otherwise the flags for the types of errors that were detected.
 */
CORE_API ECbValidateError ValidateCompactBinary(const FCbField& Value, ECbValidateMode Mode);
CORE_API ECbValidateError ValidateCompactBinary(const FCbArray& Value, ECbValidateMode Mode);
CORE_API ECbValidateError ValidateCompactBinary(const FCbObject& Value, ECbValidateMode Mode);
CORE_API ECbValidateError ValidateCompactBinary(const FCbPackage& Value, ECbValidateMode Mode);
CORE_API ECbValidateError ValidateCompactBinary(const FCbAttachment& Value, ECbValidateMode Mode);

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
