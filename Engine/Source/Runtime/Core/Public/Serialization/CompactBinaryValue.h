// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"
#include "HAL/PlatformMemory.h"
#include "IO/IoHash.h"
#include "Memory/MemoryView.h"
#include "Misc/ByteSwap.h"
#include "Misc/Guid.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/VarInt.h"

/**
 * A type that provides unchecked access to compact binary values.
 *
 * The main purpose of the type is to efficiently switch on field type. For every other use case,
 * prefer to use the field, array, and object types directly. The accessors here do not check the
 * type before reading the value, which means they can read out of bounds even on a valid compact
 * binary value if the wrong accessor is used.
 */
class FCbValue
{
public:
	constexpr FCbValue() = default;
	FCbValue(const FCbFieldView& View);
	FCbValue(ECbFieldType Type, const void* Value);

	FCbObjectView AsObjectView() const;
	FCbArrayView AsArrayView() const;

	FMemoryView AsBinary() const;
	/** Access as a string. Checks for range errors and uses the default if OutError is not null. */
	FUtf8StringView AsString(ECbFieldError* OutError = nullptr, FUtf8StringView Default = FUtf8StringView()) const;

	/**
	 * Access as an integer, with both positive and negative values returned as unsigned.
	 *
	 * Checks for range errors and uses the default if OutError is not null.
	 */
	uint64 AsInteger(UE::CompactBinary::Private::FIntegerParams Params, ECbFieldError* OutError = nullptr, uint64 Default = 0) const;

	uint64 AsIntegerPositive() const;
	int64 AsIntegerNegative() const;

	float AsFloat32() const;
	double AsFloat64() const;

	bool AsBool() const;

	FORCEINLINE FIoHash AsObjectAttachment() const { return AsHash(); }
	FORCEINLINE FIoHash AsBinaryAttachment() const { return AsHash(); }
	FORCEINLINE FIoHash AsAttachment() const { return AsHash(); }

	FIoHash AsHash() const;
	FGuid AsUuid() const;

	int64 AsDateTimeTicks() const;
	int64 AsTimeSpanTicks() const;

	FCbObjectId AsObjectId() const;

	FCbCustomById AsCustomById() const;
	FCbCustomByName AsCustomByName() const;

	FORCEINLINE ECbFieldType GetType() const { return Type; }
	FORCEINLINE const void* GetData() const { return Data; }

	FORCEINLINE explicit operator bool() const { return Type != ECbFieldType::None; }

private:
	const void* Data = nullptr;
	ECbFieldType Type = ECbFieldType::None;
};

FORCEINLINE FCbValue::FCbValue(const FCbFieldView& View)
	: FCbValue(View.GetValue())
{
}

FORCEINLINE FCbFieldView::FCbFieldView(const FCbValue& InValue)
	: TypeWithFlags(InValue.GetType())
	, Value(InValue.GetData())
{
}

FORCEINLINE FCbValue FCbFieldView::GetValue() const
{
	return FCbValue(FCbFieldType::GetType(TypeWithFlags), Value);
}

FORCEINLINE FCbValue::FCbValue(ECbFieldType InType, const void* InValue)
	: Data(InValue)
	, Type(InType)
{
}

FORCEINLINE FCbObjectView FCbValue::AsObjectView() const
{
	return FCbObjectView(*this);
}

FORCEINLINE FCbArrayView FCbValue::AsArrayView() const
{
	return FCbArrayView(*this);
}

FORCEINLINE FMemoryView FCbValue::AsBinary() const
{
	const uint8* const Bytes = static_cast<const uint8*>(Data);
	uint32 ValueSizeByteCount;
	const uint64 ValueSize = ReadVarUInt(Bytes, ValueSizeByteCount);
	return MakeMemoryView(Bytes + ValueSizeByteCount, ValueSize);
}

FORCEINLINE FUtf8StringView FCbValue::AsString(ECbFieldError* OutError, FUtf8StringView Default) const
{
	const UTF8CHAR* const Chars = static_cast<const UTF8CHAR*>(Data);
	uint32 ValueSizeByteCount;
	const uint64 ValueSize = ReadVarUInt(Chars, ValueSizeByteCount);

	if (OutError)
	{
		if (ValueSize >= (uint64(1) << 31))
		{
			*OutError = ECbFieldError::RangeError;
			return Default;
		}
		*OutError = ECbFieldError::None;
	}

	return FUtf8StringView(Chars + ValueSizeByteCount, int32(ValueSize));
}

FORCEINLINE uint64 FCbValue::AsInteger(UE::CompactBinary::Private::FIntegerParams Params, ECbFieldError* OutError, uint64 Default) const
{
	// A shift of a 64-bit value by 64 is undefined so shift by one less because magnitude is never zero.
	const uint64 OutOfRangeMask = uint64(-2) << (Params.MagnitudeBits - 1);
	const uint64 IsNegative = uint8(Type) & 1;

	uint32 MagnitudeByteCount;
	const uint64 Magnitude = ReadVarUInt(Data, MagnitudeByteCount);
	const uint64 Value = Magnitude ^ -int64(IsNegative);

	if (OutError)
	{
		const uint64 IsInRange = (!(Magnitude & OutOfRangeMask)) & ((!IsNegative) | Params.IsSigned);
		*OutError = IsInRange ? ECbFieldError::None : ECbFieldError::RangeError;

		const uint64 UseValueMask = -int64(IsInRange);
		return (Value & UseValueMask) | (Default & ~UseValueMask);
	}

	return Value;
}

FORCEINLINE uint64 FCbValue::AsIntegerPositive() const
{
	uint32 MagnitudeByteCount;
	return ReadVarUInt(Data, MagnitudeByteCount);
}

FORCEINLINE int64 FCbValue::AsIntegerNegative() const
{
	uint32 MagnitudeByteCount;
	return int64(ReadVarUInt(Data, MagnitudeByteCount)) ^ -int64(1);
}

FORCEINLINE float FCbValue::AsFloat32() const
{
	const uint32 Value = NETWORK_ORDER32(FPlatformMemory::ReadUnaligned<uint32>(Data));
	return reinterpret_cast<const float&>(Value);
}

FORCEINLINE double FCbValue::AsFloat64() const
{
	const uint64 Value = NETWORK_ORDER64(FPlatformMemory::ReadUnaligned<uint64>(Data));
	return reinterpret_cast<const double&>(Value);
}

FORCEINLINE bool FCbValue::AsBool() const
{
	return uint8(Type) & 1;
}

FORCEINLINE FIoHash FCbValue::AsHash() const
{
	return FIoHash(*static_cast<const FIoHash::ByteArray*>(Data));
}

FORCEINLINE FGuid FCbValue::AsUuid() const
{
	FGuid Value = FPlatformMemory::ReadUnaligned<FGuid>(Data);
	Value.A = NETWORK_ORDER32(Value.A);
	Value.B = NETWORK_ORDER32(Value.B);
	Value.C = NETWORK_ORDER32(Value.C);
	Value.D = NETWORK_ORDER32(Value.D);
	return Value;
}

FORCEINLINE int64 FCbValue::AsDateTimeTicks() const
{
	return NETWORK_ORDER64(FPlatformMemory::ReadUnaligned<int64>(Data));
}

FORCEINLINE int64 FCbValue::AsTimeSpanTicks() const
{
	return NETWORK_ORDER64(FPlatformMemory::ReadUnaligned<int64>(Data));
}

FORCEINLINE FCbObjectId FCbValue::AsObjectId() const
{
	return FCbObjectId(MakeMemoryView(Data, sizeof(FCbObjectId)));
}

FORCEINLINE FCbCustomById FCbValue::AsCustomById() const
{
	const uint8* Bytes = static_cast<const uint8*>(Data);
	uint32 DataSizeByteCount;
	const uint64 DataSize = ReadVarUInt(Bytes, DataSizeByteCount);
	Bytes += DataSizeByteCount;

	FCbCustomById Value;
	uint32 TypeIdByteCount;
	Value.Id = ReadVarUInt(Bytes, TypeIdByteCount);
	Value.Data = MakeMemoryView(Bytes + TypeIdByteCount, DataSize - TypeIdByteCount);
	return Value;
}

FORCEINLINE FCbCustomByName FCbValue::AsCustomByName() const
{
	const uint8* Bytes = static_cast<const uint8*>(Data);
	uint32 DataSizeByteCount;
	const uint64 DataSize = ReadVarUInt(Bytes, DataSizeByteCount);
	Bytes += DataSizeByteCount;

	uint32 TypeNameLenByteCount;
	const uint64 TypeNameLen = ReadVarUInt(Bytes, TypeNameLenByteCount);
	Bytes += TypeNameLenByteCount;

	FCbCustomByName Value;
	Value.Name = FUtf8StringView(
		reinterpret_cast<const UTF8CHAR*>(Bytes),
		static_cast<int32>(TypeNameLen));
	Value.Data = MakeMemoryView(Bytes + TypeNameLen, DataSize - TypeNameLen - TypeNameLenByteCount);
	return Value;
}
