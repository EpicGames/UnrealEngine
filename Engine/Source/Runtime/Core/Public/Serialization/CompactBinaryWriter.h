// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Memory/MemoryFwd.h"
#include "Memory/MemoryView.h"
#include "Serialization/CompactBinary.h"

#include <type_traits>

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FArchive;
class FCbAttachment;
class FName;
struct FDateTime;
struct FGuid;
struct FTimespan;

/**
 * A writer for compact binary object, arrays, and fields.
 *
 * The writer produces a sequence of fields that can be saved to a provided memory buffer or into
 * a new owned buffer. The typical use case is to write a single object, which can be accessed by
 * calling Save().AsObject() or Save(Buffer).AsObjectView().
 *
 * The writer will assert on most incorrect usage and will always produce valid compact binary if
 * provided with valid input. The writer does not check for invalid UTF-8 string encoding, object
 * fields with duplicate names, or invalid compact binary being copied from another source.
 *
 * It is most convenient to use the streaming API for the writer, as demonstrated in the example.
 *
 * When writing a small amount of compact binary data, TCbWriter can be more efficient as it uses
 * a fixed-size stack buffer for storage before spilling onto the heap.
 *
 * @see TCbWriter
 *
 * Example:
 *
 * FCbObject WriteObject()
 * {
 *     TCbWriter<256> Writer;
 *     Writer.BeginObject();
 * 
 *     Writer << "Resize" << true;
 *     Writer << "MaxWidth" << 1024;
 *     Writer << "MaxHeight" << 1024;
 * 
 *     Writer.BeginArray();
 *     Writer << "FormatA" << "FormatB" << "FormatC";
 *     Writer.EndArray();
 * 
 *     Writer.EndObject();
 *     return Writer.Save().AsObject();
 * }
 */
class FCbWriter
{
public:
	CORE_API FCbWriter();
	CORE_API ~FCbWriter();

	FCbWriter(const FCbWriter&) = delete;
	FCbWriter& operator=(const FCbWriter&) = delete;

	/** Empty the writer without releasing any allocated memory. */
	CORE_API void Reset();

	/**
	 * Serialize the field(s) to an owned buffer and return it as an iterator.
	 *
	 * It is not valid to call this function in the middle of writing an object, array, or field.
	 * The writer remains valid for further use when this function returns.
	 */
	CORE_API FCbFieldIterator Save() const;

	/**
	 * Serialize the field(s) to memory.
	 *
	 * It is not valid to call this function in the middle of writing an object, array, or field.
	 * The writer remains valid for further use when this function returns.
	 *
	 * @param Buffer A mutable memory view to write to. Must be exactly GetSaveSize() bytes.
	 * @return An iterator for the field(s) written to the buffer.
	 */
	CORE_API FCbFieldViewIterator Save(FMutableMemoryView Buffer) const;

	/**
	 * Serialize the field(s) to an archive.
	 *
	 * It is not valid to call this function in the middle of writing an object, array, or field.
	 * The writer remains valid for further use when this function returns.
	 *
	 * @param Ar An archive to write to. Exactly GetSaveSize() bytes will be written.
	 */
	CORE_API void Save(FArchive& Ar) const;

	/**
	 * The size of buffer (in bytes) required to serialize the fields that have been written.
	 *
	 * It is not valid to call this function in the middle of writing an object, array, or field.
	 */
	CORE_API uint64 GetSaveSize() const;

	/**
	 * Sets the name of the next field to be written.
	 *
	 * It is not valid to call this function when writing a field inside an array.
	 * Names must be valid UTF-8 and must be unique within an object.
	 */
	CORE_API FCbWriter& SetName(FUtf8StringView Name);

	/** Copy the value (not the name) of an existing field. */
	inline void AddField(FUtf8StringView Name, const FCbFieldView& Value) { SetName(Name); AddField(Value); }
	CORE_API void AddField(const FCbFieldView& Value);
	/** Copy the value (not the name) of an existing field. Holds a reference if owned. */
	inline void AddField(FUtf8StringView Name, const FCbField& Value) { SetName(Name); AddField(Value); }
	CORE_API void AddField(const FCbField& Value);

	/** Begin a new object. Must have a matching call to EndObject. */
	inline void BeginObject(FUtf8StringView Name) { SetName(Name); BeginObject(); }
	CORE_API void BeginObject();
	/** End an object after its fields have been written. */
	CORE_API void EndObject();

	/** Copy the value (not the name) of an existing object. */
	inline void AddObject(FUtf8StringView Name, const FCbObjectView& Value) { SetName(Name); AddObject(Value); }
	CORE_API void AddObject(const FCbObjectView& Value);
	/** Copy the value (not the name) of an existing object. Holds a reference if owned. */
	inline void AddObject(FUtf8StringView Name, const FCbObject& Value) { SetName(Name); AddObject(Value); }
	CORE_API void AddObject(const FCbObject& Value);

	/** Begin a new array. Must have a matching call to EndArray. */
	inline void BeginArray(FUtf8StringView Name) { SetName(Name); BeginArray(); }
	CORE_API void BeginArray();
	/** End an array after its fields have been written. */
	CORE_API void EndArray();

	/** Copy the value (not the name) of an existing array. */
	inline void AddArray(FUtf8StringView Name, const FCbArrayView& Value) { SetName(Name); AddArray(Value); }
	CORE_API void AddArray(const FCbArrayView& Value);
	/** Copy the value (not the name) of an existing array. Holds a reference if owned. */
	inline void AddArray(FUtf8StringView Name, const FCbArray& Value) { SetName(Name); AddArray(Value); }
	CORE_API void AddArray(const FCbArray& Value);

	/** Write a null field. */
	inline void AddNull(FUtf8StringView Name) { SetName(Name); AddNull(); }
	CORE_API void AddNull();

	/** Write a binary field by copying Size bytes from Value. */
	inline void AddBinary(FUtf8StringView Name, const void* Value, uint64 Size) { SetName(Name); AddBinary(Value, Size); }
	CORE_API void AddBinary(const void* Value, uint64 Size);
	/** Write a binary field by copying the view. */
	inline void AddBinary(FUtf8StringView Name, FMemoryView Value) { SetName(Name); AddBinary(Value); }
	inline void AddBinary(FMemoryView Value) { AddBinary(Value.GetData(), Value.GetSize()); }
	/** Write a binary field by copying the buffer. Holds a reference if owned. */
	CORE_API void AddBinary(FUtf8StringView Name, const FSharedBuffer& Value);
	CORE_API void AddBinary(const FSharedBuffer& Value);
	CORE_API void AddBinary(FUtf8StringView Name, const FCompositeBuffer& Value);
	CORE_API void AddBinary(const FCompositeBuffer& Value);

	/** Write a string field by copying the UTF-8 value. */
	inline void AddString(FUtf8StringView Name, FUtf8StringView Value) { SetName(Name); AddString(Value); }
	CORE_API void AddString(FUtf8StringView Value);
	/** Write a string field by converting the UTF-16 value to UTF-8. */
	inline void AddString(FUtf8StringView Name, FWideStringView Value) { SetName(Name); AddString(Value); }
	CORE_API void AddString(FWideStringView Value);

	/** Write an integer field. */
	inline void AddInteger(FUtf8StringView Name, int32 Value) { SetName(Name); AddInteger(Value); }
	CORE_API void AddInteger(int32 Value);
	/** Write an integer field. */
	inline void AddInteger(FUtf8StringView Name, int64 Value) { SetName(Name); AddInteger(Value); }
	CORE_API void AddInteger(int64 Value);
	/** Write an integer field. */
	inline void AddInteger(FUtf8StringView Name, uint32 Value) { SetName(Name); AddInteger(Value); }
	CORE_API void AddInteger(uint32 Value);
	/** Write an integer field. */
	inline void AddInteger(FUtf8StringView Name, uint64 Value) { SetName(Name); AddInteger(Value); }
	CORE_API void AddInteger(uint64 Value);

	/** Write a float field from a 32-bit float value. */
	inline void AddFloat(FUtf8StringView Name, float Value) { SetName(Name); AddFloat(Value); }
	CORE_API void AddFloat(float Value);
	/** Write a float field from a 64-bit float value. */
	inline void AddFloat(FUtf8StringView Name, double Value) { SetName(Name); AddFloat(Value); }
	CORE_API void AddFloat(double Value);

	/** Write a bool field. */
	inline void AddBool(FUtf8StringView Name, bool bValue) { SetName(Name); AddBool(bValue); }
	CORE_API void AddBool(bool bValue);

	/** Write a field referencing an object attachment by its hash. */
	inline void AddObjectAttachment(FUtf8StringView Name, const FIoHash& Value) { SetName(Name); AddObjectAttachment(Value); }
	CORE_API void AddObjectAttachment(const FIoHash& Value);
	/** Write a field referencing a binary attachment by its hash. */
	inline void AddBinaryAttachment(FUtf8StringView Name, const FIoHash& Value) { SetName(Name); AddBinaryAttachment(Value); }
	CORE_API void AddBinaryAttachment(const FIoHash& Value);
	/** Write a field referencing the attachment by its hash. */
	inline void AddAttachment(FUtf8StringView Name, const FCbAttachment& Attachment) { SetName(Name); AddAttachment(Attachment); }
	CORE_API void AddAttachment(const FCbAttachment& Attachment);

	/** Write a hash field. */
	inline void AddHash(FUtf8StringView Name, const FIoHash& Value) { SetName(Name); AddHash(Value); }
	CORE_API void AddHash(const FIoHash& Value);
	/** Write a UUID field. */
	inline void AddUuid(FUtf8StringView Name, const FGuid& Value) { SetName(Name); AddUuid(Value); }
	CORE_API void AddUuid(const FGuid& Value);

	/** Write a date/time field with the specified count of 100ns ticks since the epoch. */
	inline void AddDateTimeTicks(FUtf8StringView Name, int64 Ticks) { SetName(Name); AddDateTimeTicks(Ticks); }
	CORE_API void AddDateTimeTicks(int64 Ticks);

	/** Write a date/time field. */
	CORE_API void AddDateTime(FUtf8StringView Name, FDateTime Value);
	CORE_API void AddDateTime(FDateTime Value);

	/** Write a time span field with the specified count of 100ns ticks. */
	inline void AddTimeSpanTicks(FUtf8StringView Name, int64 Ticks) { SetName(Name); AddTimeSpanTicks(Ticks); }
	CORE_API void AddTimeSpanTicks(int64 Ticks);

	/** Write a time span field. */
	CORE_API void AddTimeSpan(FUtf8StringView Name, FTimespan Value);
	CORE_API void AddTimeSpan(FTimespan Value);

	/** Write an ObjectId field. */
	inline void AddObjectId(FUtf8StringView Name, const FCbObjectId& Value) { SetName(Name); AddObjectId(Value); }
	CORE_API void AddObjectId(const FCbObjectId& Value);

	/** Write a custom field with an integer sub-type identifier. */
	inline void AddCustom(FUtf8StringView FieldName, uint64 TypeId, FMemoryView Value) { SetName(FieldName); AddCustom(TypeId, Value); }
	CORE_API void AddCustom(uint64 TypeId, FMemoryView Value);

	/** Write a custom field with a string sub-type identifier. */
	inline void AddCustom(FUtf8StringView FieldName, FUtf8StringView TypeName, FMemoryView Value) { SetName(FieldName); AddCustom(TypeName, Value); }
	CORE_API void AddCustom(FUtf8StringView TypeName, FMemoryView Value);

	/** Private flags that are public to work with ENUM_CLASS_FLAGS. */
	enum class EStateFlags : uint8;

protected:
	/** Reserve the specified size up front until the format is optimized. */
	CORE_API explicit FCbWriter(int64 InitialSize);

private:

	/** Begin writing a field. May be called twice for named fields. */
	void BeginField();

	/** Finish writing a field by writing its type. */
	void EndField(ECbFieldType Type);

	/** Set the field name if valid in this state, otherwise write add a string field. */
	CORE_API void SetNameOrAddString(FUtf8StringView NameOrValue);

	/** Returns a view of the name of the active field, if any, otherwise the empty view. */
	FUtf8StringView GetActiveName() const;

	/** Remove field types after the first to make the sequence uniform. */
	void MakeFieldsUniform(int64 FieldBeginOffset, int64 FieldEndOffset);

	/** State of the object, array, or top-level field being written. */
	struct FState
	{
		EStateFlags Flags{};
		/** The type of the fields in the sequence if uniform, otherwise None. */
		ECbFieldType UniformType{};
		/** The offset of the start of the current field. */
		int64 Offset{};
		/** The number of fields written in this state. */
		uint64 Count{};
	};

private:
	// This is a prototype-quality format for the writer. Using an array of bytes is inefficient,
	// and will lead to many unnecessary copies and moves of the data to resize the array, insert
	// object and array sizes, and remove field types for uniform objects and uniform arrays. The
	// optimized format will be a list of power-of-two blocks and an optional first block that is
	// provided externally, such as on the stack. That format will store the offsets that require
	// object or array sizes to be inserted and field types to be removed, and will perform those
	// operations only when saving to a buffer.
	TArray64<uint8> Data;
	TArray<FState> States;

public:
	/** Write the field name if valid in this state, otherwise write the string value. */
	inline FCbWriter& operator<<(FUtf8StringView NameOrValue)
	{
		SetNameOrAddString(NameOrValue);
		return *this;
	}

	/** Write the field name if valid in this state, otherwise write the string value. */
	inline FCbWriter& operator<<(const ANSICHAR* NameOrValue)
	{
		return *this << FAnsiStringView(NameOrValue);
	}

	/** Write the field name if valid in this state, otherwise write the string value. */
	inline FCbWriter& operator<<(const UTF8CHAR* NameOrValue)
	{
		return *this << FUtf8StringView(NameOrValue);
	}

	inline FCbWriter& operator<<(const FCbFieldView& Value)
	{
		AddField(Value);
		return *this;
	}

	inline FCbWriter& operator<<(const FCbField& Value)
	{
		AddField(Value);
		return *this;
	}

	inline FCbWriter& operator<<(const FCbObjectView& Value)
	{
		AddObject(Value);
		return *this;
	}

	inline FCbWriter& operator<<(const FCbObject& Value)
	{
		AddObject(Value);
		return *this;
	}

	inline FCbWriter& operator<<(const FCbArrayView& Value)
	{
		AddArray(Value);
		return *this;
	}

	inline FCbWriter& operator<<(const FCbArray& Value)
	{
		AddArray(Value);
		return *this;
	}

	inline FCbWriter& operator<<(TYPE_OF_NULLPTR)
	{
		AddNull();
		return *this;
	}

	inline FCbWriter& operator<<(FWideStringView Value)
	{
		AddString(Value);
		return *this;
	}

	inline FCbWriter& operator<<(const WIDECHAR* Value)
	{
		AddString(Value);
		return *this;
	}

	inline FCbWriter& operator<<(int32 Value)
	{
		AddInteger(Value);
		return *this;
	}

	inline FCbWriter& operator<<(int64 Value)
	{
		AddInteger(Value);
		return *this;
	}

	inline FCbWriter& operator<<(uint32 Value)
	{
		AddInteger(Value);
		return *this;
	}

	inline FCbWriter& operator<<(uint64 Value)
	{
		AddInteger(Value);
		return *this;
	}

#if PLATFORM_LINUX || PLATFORM_APPLE
	// On Unix-like platforms, size_t is an unsigned long, whereas UE defines uint64 as unsigned long long,
	// making the two types immiscible for template resolution. However, SSIZE_T is currently defined as
	// int64 (see FGenericPlatformTypes) and so no special overload is required for that type.
	inline FCbWriter& operator<<(SIZE_T Value)
	{
		AddInteger(static_cast<uint64>(Value));
		return *this;
	}
#endif

	inline FCbWriter& operator<<(float Value)
	{
		AddFloat(Value);
		return *this;
	}

	inline FCbWriter& operator<<(double Value)
	{
		AddFloat(Value);
		return *this;
	}

	inline FCbWriter& operator<<(bool Value)
	{
		AddBool(Value);
		return *this;
	}

	inline FCbWriter& operator<<(const FCbAttachment& Attachment)
	{
		AddAttachment(Attachment);
		return *this;
	}

	inline FCbWriter& operator<<(const FIoHash& Value)
	{
		AddHash(Value);
		return *this;
	}

	inline FCbWriter& operator<<(const FGuid& Value)
	{
		AddUuid(Value);
		return *this;
	}

	CORE_API FCbWriter& operator<<(FDateTime Value);
	CORE_API FCbWriter& operator<<(FTimespan Value);

	inline FCbWriter& operator<<(const FCbObjectId& Value)
	{
		AddObjectId(Value);
		return *this;
	}

	CORE_API FCbWriter& operator<<(const FString& Value);
	CORE_API FCbWriter& operator<<(const FAnsiString& Value);
	CORE_API FCbWriter& operator<<(const FUtf8String& Value);
	CORE_API FCbWriter& operator<<(FName Value);

	template <typename T, typename Allocator,
		std::void_t<decltype(std::declval<FCbWriter&>() << std::declval<const T&>())>* = nullptr>
	inline FCbWriter& operator<<(const TArray<T, Allocator>& Value)
	{
		BeginArray();
		for (const T& Element : Value)
		{
			*this << Element;
		}
		EndArray();
		return *this;
	}
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * A writer for compact binary object, arrays, and fields that uses a fixed-size stack buffer.
 *
 * @see FCbWriter
 */
template <uint32 InlineBufferSize>
class TCbWriter : public FCbWriter
{
public:
	inline TCbWriter()
		: FCbWriter(InlineBufferSize)
	{
	}

	TCbWriter(const TCbWriter&) = delete;
	TCbWriter& operator=(const TCbWriter&) = delete;

private:
	// Reserve the inline buffer now even though we are unable to use it. This will avoid causing
	// new stack overflows when this functionality is properly implemented in the future.
	uint8 Buffer[InlineBufferSize];
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
