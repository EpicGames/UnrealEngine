// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "CoreTypes.h"

class FArchive;

/**
 * Implements a helper structure for compression support
 *
 * This structure contains information on the compressed and uncompressed size of a chunk of data.
 */
struct FCompressedChunkInfo
{
	/** Holds the data's compressed size. */
	int64 CompressedSize = 0;

	/** Holds the data's uncompresses size. */
	int64 UncompressedSize = 0;

	/**
	 * Serializes an FCompressedChunkInfo value from or into an archive.
	 *
	 * @param Ar The archive to serialize from or to.
	 * @param Value The value to serialize.
	 */
	friend CORE_API FArchive& operator<<(FArchive& Ar, FCompressedChunkInfo& Value);

};
