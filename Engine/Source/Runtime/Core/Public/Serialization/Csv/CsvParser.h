// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"

/** A simple, efficient csv parser */
struct FCsvParser
{
	typedef TArray<TArray<const TCHAR*>> FRows;

	/**
	 *	Construct with a string.
	 *	Takes a copy because the parser tramples over the file to create null-terminated cell strings
	 *	Avoid a copy by passing overship of the source string with MoveTemp()
	 */
	CORE_API FCsvParser(FString&& InSourceString);
	CORE_API FCsvParser(const FString& InSourceString);

	/**
	 *	Const access to the parsed rows
	 *	Rows are stored as arrays of parsed, null-terminated cell-strings.
	 *	Object (and contained strings) is valid for the lifetime of the parser.
	 */
	const FRows& GetRows() const
	{
		return Rows;
	}

	/** Non copyable - has pointers to itself */
	FCsvParser(FCsvParser&&) = delete;
	FCsvParser(const FCsvParser&) = delete;
	FCsvParser& operator=(FCsvParser&&) = delete;
	FCsvParser& operator=(const FCsvParser&) = delete;

private:
	/** Internal enum to control parsing progress */
	enum struct EParseResult { EndOfCell, EndOfRow, EndOfString };

	/** Parse all the rows in the source data */
	void ParseRows();

	/** Parse a single row at the current read position */
	EParseResult ParseRow();

	/** Parse a cell at the current read position */
	EParseResult ParseCell();

private:

	/** Raw csv buffer, sliced and diced by the parser */
	FString SourceString;

	/** Fixed pointer to the start of the buffer */
	TCHAR* BufferStart;

	/** Current parser read position */
	const TCHAR* ReadAt;

	/** Array of cell string arrays */
	FRows Rows;

private:

	/** Helper to measure the size of a new line at the current read position */
	int8 MeasureNewLine(const TCHAR* At);
};
