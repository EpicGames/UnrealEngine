// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Misc/Crc.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

class FArchive;
class FCustomVersionContainer;
struct FCustomVersion;

enum class ECustomVersionSerializationFormat : uint8
{
	Unknown,
	Guids,
	Enums,
	Optimized,

	// Add new versions above this comment
	CustomVersion_Automatic_Plus_One,
	Latest = CustomVersion_Automatic_Plus_One - 1
};

typedef TArray<FCustomVersion> FCustomVersionArray;
typedef bool (*CustomVersionValidatorFunc)(const FCustomVersion& Version, const FCustomVersionArray& AllVersions, const TCHAR* DebugContext);

/**
 * Structure to hold unique custom key with its version.
 */
struct FCustomVersion
{
	friend class FCustomVersionContainer;

	/** Unique custom key. */
	FGuid Key;

	/** Custom version. */
	int32 Version;

	/** Number of times this GUID has been registered */
	int32 ReferenceCount;

	/** An optional validator that will be called if a package has a given version that can prevent it from loading */
	CustomVersionValidatorFunc Validator;

	/** Constructor. */
	UE_FORCEINLINE_HINT FCustomVersion()
	: Validator(nullptr)
	{
	}

	/** Helper constructor. */
	inline FCustomVersion(FGuid InKey, int32 InVersion, FName InFriendlyName, CustomVersionValidatorFunc InValidatorFunc = nullptr)
	: Key           (InKey)
	, Version       (InVersion)
	, ReferenceCount(1)
	, Validator     (InValidatorFunc)
	, FriendlyName  (InFriendlyName)
	{
	}

	/** Equality comparison operator for Key */
	UE_FORCEINLINE_HINT bool operator==(FGuid InKey) const
	{
		return Key == InKey;
	}

	/** Inequality comparison operator for Key */
	UE_FORCEINLINE_HINT bool operator!=(FGuid InKey) const
	{
		return Key != InKey;
	}

	CORE_API friend FArchive& operator<<(FArchive& Ar, FCustomVersion& Version);
	CORE_API friend void operator<<(FStructuredArchive::FSlot Slot, FCustomVersion& Version);

	/** Gets the friendly name for error messages or whatever */
	CORE_API const FName GetFriendlyName() const;

private:

	/** Friendly name for error messages or whatever. Lazy initialized for serialized versions */
	mutable FName FriendlyName;
};

class FCustomVersionRegistration;

/** Flags defining how the Custom Version should be applied */
enum class ESetCustomVersionFlags : uint8
{
	None,

	// If Set, it will not query the versions registry to update the version data if the key is already in the container 
	SkipUpdateExistingVersion = 0x01,
};
ENUM_CLASS_FLAGS(ESetCustomVersionFlags)

/**
 * Container for all available/serialized custom versions.
 */
class FCustomVersionContainer
{
	friend struct FStaticCustomVersionRegistry;

public:
	/** Gets available custom versions in this container. */
	UE_FORCEINLINE_HINT const FCustomVersionArray& GetAllVersions() const
	{
		return Versions;
	}

	/**
	 * Gets a custom version from the container.
	 *
	 * @param CustomKey Custom key for which to retrieve the version.
	 * @return The FCustomVersion for the specified custom key, or nullptr if the key doesn't exist in the container.
	 */
	CORE_API const FCustomVersion* GetVersion(FGuid CustomKey) const;

	/**
	* Gets a custom version friendly name from the container.
	*
	* @param CustomKey Custom key for which to retrieve the version.
	* @return The friendly name for the specified custom key, or NAME_None if the key doesn't exist in the container.
	*/
	CORE_API const FName GetFriendlyName(FGuid CustomKey) const;

	/**
	 * Sets a specific custom version in the container.
	 *
	 * @param CustomKey Custom key for which to retrieve the version.
	 * @param Version The version number for the specified custom key
	 * @param FriendlyName A friendly name to assign to this version
	 */
	CORE_API void SetVersion(FGuid CustomKey, int32 Version, FName FriendlyName);

	/**
	 * Sets a specific custom version in the container. It queries the versions registry to get the version data for the provided key
	 * @param CustomKey Custom key for which to retrieve the version.
	 * @param Options Optional flags used to alter the behavior of this method
	 */
	CORE_API void SetVersionUsingRegistry(FGuid CustomKey, ESetCustomVersionFlags Options = ESetCustomVersionFlags::None);

	/** Serialization. */
	CORE_API void Serialize(FArchive& Ar, ECustomVersionSerializationFormat Format = ECustomVersionSerializationFormat::Latest);
	CORE_API void Serialize(FStructuredArchive::FSlot Slot, ECustomVersionSerializationFormat Format = ECustomVersionSerializationFormat::Latest);

	/**
	 * Empties the custom version container.
	 */
	CORE_API void Empty();

	/**
	 * Sorts the custom version container by key.
	 */
	CORE_API void SortByKey();

	/** Return a string representation of custom versions. Used for debug. */
	CORE_API FString ToString(const FString& Indent) const;

private:

	/** Array containing custom versions. */
	FCustomVersionArray Versions;

};

enum class ECustomVersionDifference { Missing, Newer, Older, Invalid };

struct FCustomVersionDifference
{
	ECustomVersionDifference Type;
	const FCustomVersion* Version;
};

/** Provides access to code-defined custom versions registered via FCustomVersionRegistration. */
class FCurrentCustomVersions
{
public:
	/** Get a copy of all versions that has been statically registered so far in the module loading process. */
	static CORE_API FCustomVersionContainer GetAll();

	/** Get a copy of a single statically registered version if it exists. */
	static CORE_API TOptional<FCustomVersion> Get(const FGuid& Guid);

	/** Compare a number of versions to current ones and return potential differences. */
	static CORE_API TArray<FCustomVersionDifference> Compare(const FCustomVersionArray& CompareVersions, const TCHAR* DebugContext);

private:
	friend class FCustomVersionRegistration;

	static CORE_API void Register(const FGuid& Key, int32 Version, const TCHAR* FriendlyName, CustomVersionValidatorFunc ValidatorFunc);
	static CORE_API void Unregister(const FGuid& Key);
};


/**
 * This class will cause the registration of a custom version number and key with the global
 * FCustomVersionContainer when instantiated, and unregister when destroyed.  It should be
 * instantiated as a global variable somewhere in the module which needs a custom version number.
 */
class FCustomVersionRegistration : FNoncopyable
{
public:
	/** @param InFriendlyName must be a string literal */
	template<int N>
	FCustomVersionRegistration(FGuid InKey, int32 Version, const TCHAR(&InFriendlyName)[N], CustomVersionValidatorFunc InValidatorFunc = nullptr)
	: Key(InKey)
	{
		FCurrentCustomVersions::Register(InKey, Version, InFriendlyName, InValidatorFunc);
	}

	~FCustomVersionRegistration()
	{
		FCurrentCustomVersions::Unregister(Key);
	}

private:
	FGuid Key;
};
