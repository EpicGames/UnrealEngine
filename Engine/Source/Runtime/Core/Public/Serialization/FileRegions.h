// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "PixelFormat.h"

class FCbFieldView;
class FCbWriter;

// Describes the type of data found in a given file region.
enum class EFileRegionType : uint8
{
	// NOTE: Enum values here must match those in AutomationUtils/FileRegions.cs
	None = 0,

	// @todo re-number these and bump texture DDC key
	// Texture data types
	BC1 = 2,
	BC2 = 5,
	BC3 = 6,
	BC4 = 3,
	BC5 = 7
};

// Represents a region of logically related bytes within a larger block of cooked data.
// Regions are used to improve data compression and patching on some platforms.
struct FFileRegion
{
	static CORE_API const TCHAR* RegionsFileExtension;

	uint64 Offset;
	uint64 Length;
	EFileRegionType Type;

	FFileRegion()
		: Offset(0)
		, Length(0)
		, Type(EFileRegionType::None)
	{}

	FFileRegion(uint64 InOffset, uint64 InLength, EFileRegionType InType)
		: Offset(InOffset)
		, Length(InLength)
		, Type(InType)
	{}

	static CORE_API void AccumulateFileRegions(TArray<FFileRegion>& InOutRegions, int64 EntryOffset, int64 PayloadOffset, int64 EndOffset, TArrayView<const FFileRegion> InnerFileRegions);
	static CORE_API void SerializeFileRegions(class FArchive& Ar, TArray<FFileRegion>& Regions);

	CORE_API friend FCbWriter& operator<<(FCbWriter& Writer, const FFileRegion& Region);
	static CORE_API bool LoadFromCompactBinary(FCbFieldView Obj, FFileRegion& OutRegion);

	static inline EFileRegionType SelectType(EPixelFormat Format)
	{
		switch (Format)
		{
		case PF_DXT1: return EFileRegionType::BC1;
		case PF_DXT3: return EFileRegionType::BC2;
		case PF_DXT5: return EFileRegionType::BC3;
		case PF_BC4:  return EFileRegionType::BC4;
		case PF_BC5:  return EFileRegionType::BC5;
		}

		return EFileRegionType::None;
	}
};
