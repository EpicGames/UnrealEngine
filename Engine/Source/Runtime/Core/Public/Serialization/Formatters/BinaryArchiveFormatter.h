// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchiveFormatter.h"
#include "Serialization/StructuredArchiveNameHelpers.h"

class FName;
class FString;
class FText;
class UObject;
struct FLazyObjectPtr;
struct FObjectPtr;
struct FSoftObjectPath;
struct FSoftObjectPtr;
struct FWeakObjectPtr;

class FBinaryArchiveFormatter final : public FStructuredArchiveFormatter
{
public:
	CORE_API FBinaryArchiveFormatter(FArchive& InInner);
	CORE_API virtual ~FBinaryArchiveFormatter();

	CORE_API virtual bool HasDocumentTree() const override;
	virtual FArchive& GetUnderlyingArchive() override;

	virtual void EnterRecord() override;
	virtual void LeaveRecord() override;
	virtual void EnterField(FArchiveFieldName Name) override;
	virtual void LeaveField() override;
	CORE_API virtual bool TryEnterField(FArchiveFieldName Name, bool bEnterIfWriting) override;

	virtual void EnterArray(int32& NumElements) override;
	virtual void LeaveArray() override;
	virtual void EnterArrayElement() override;
	virtual void LeaveArrayElement() override;

	virtual void EnterStream() override;
	virtual void LeaveStream() override;
	virtual void EnterStreamElement() override;
	virtual void LeaveStreamElement() override;

	virtual void EnterMap(int32& NumElements) override;
	virtual void LeaveMap() override;
	virtual void EnterMapElement(FString& Name) override;
	virtual void LeaveMapElement() override;

	virtual void EnterAttributedValue() override;
	virtual void EnterAttribute(FArchiveFieldName AttributeName) override;
	virtual void EnterAttributedValueValue() override;
	virtual void LeaveAttribute() override;
	virtual void LeaveAttributedValue() override;
	virtual bool TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenWriting) override;
	virtual bool TryEnterAttributedValueValue() override;

	virtual void Serialize(uint8& Value) override;
	virtual void Serialize(uint16& Value) override;
	virtual void Serialize(uint32& Value) override;
	virtual void Serialize(uint64& Value) override;
	virtual void Serialize(int8& Value) override;
	virtual void Serialize(int16& Value) override;
	virtual void Serialize(int32& Value) override;
	virtual void Serialize(int64& Value) override;
	virtual void Serialize(float& Value) override;
	virtual void Serialize(double& Value) override;
	virtual void Serialize(bool& Value) override;
	virtual void Serialize(UTF32CHAR& Value) override;
	virtual void Serialize(FString& Value) override;
	virtual void Serialize(FName& Value) override;
	virtual void Serialize(UObject*& Value) override;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual void Serialize(Verse::VCell*& Value) override;
#endif
	virtual void Serialize(FText& Value) override;
	virtual void Serialize(FWeakObjectPtr& Value) override;
	virtual void Serialize(FSoftObjectPtr& Value) override;
	virtual void Serialize(FSoftObjectPath& Value) override;
	virtual void Serialize(FLazyObjectPtr& Value) override;
	virtual void Serialize(FObjectPtr& Value) override;
	CORE_API virtual void Serialize(TArray<uint8>& Value) override;
	virtual void Serialize(void* Data, uint64 DataSize) override;

private:

	FArchive& Inner;
};

inline FArchive& FBinaryArchiveFormatter::GetUnderlyingArchive()
{
	return Inner;
}

inline void FBinaryArchiveFormatter::EnterRecord()
{
}

inline void FBinaryArchiveFormatter::LeaveRecord()
{
}

inline void FBinaryArchiveFormatter::EnterField(FArchiveFieldName Name)
{
}

inline void FBinaryArchiveFormatter::LeaveField()
{
}

inline bool FBinaryArchiveFormatter::TryEnterField(FArchiveFieldName Name, bool bEnterWhenWriting)
{
	bool bValue = bEnterWhenWriting;
	Inner << bValue;
	if (bValue)
	{
		EnterField(Name);
	}
	return bValue;
}

inline void FBinaryArchiveFormatter::EnterArray(int32& NumElements)
{
	Inner << NumElements;
}

inline void FBinaryArchiveFormatter::LeaveArray()
{
}

inline void FBinaryArchiveFormatter::EnterArrayElement()
{
}

inline void FBinaryArchiveFormatter::LeaveArrayElement()
{
}

inline void FBinaryArchiveFormatter::EnterStream()
{
}

inline void FBinaryArchiveFormatter::LeaveStream()
{
}

inline void FBinaryArchiveFormatter::EnterStreamElement()
{
}

inline void FBinaryArchiveFormatter::LeaveStreamElement()
{
}

inline void FBinaryArchiveFormatter::EnterMap(int32& NumElements)
{
	Inner << NumElements;
}

inline void FBinaryArchiveFormatter::LeaveMap()
{
}

inline void FBinaryArchiveFormatter::EnterMapElement(FString& Name)
{
	Inner << Name;
}

inline void FBinaryArchiveFormatter::LeaveMapElement()
{
}

inline void FBinaryArchiveFormatter::EnterAttributedValue()
{
}

inline void FBinaryArchiveFormatter::EnterAttribute(FArchiveFieldName AttributeName)
{
}

inline void FBinaryArchiveFormatter::EnterAttributedValueValue()
{
}

inline bool FBinaryArchiveFormatter::TryEnterAttributedValueValue()
{
	return false;
}

inline void FBinaryArchiveFormatter::LeaveAttribute()
{
}

inline void FBinaryArchiveFormatter::LeaveAttributedValue()
{
}

inline bool FBinaryArchiveFormatter::TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenWriting)
{
	bool bValue = bEnterWhenWriting;
	Inner << bValue;
	if (bValue)
	{
		EnterAttribute(AttributeName);
	}
	return bValue;
}

inline void FBinaryArchiveFormatter::Serialize(uint8& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(uint16& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(uint32& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(uint64& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(int8& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(int16& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(int32& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(int64& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(float& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(double& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(bool& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(UTF32CHAR& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(FString& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(FName& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(UObject*& Value)
{
	Inner << Value;
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
inline void FBinaryArchiveFormatter::Serialize(Verse::VCell*& Value)
{
	Inner << Value;
}
#endif

inline void FBinaryArchiveFormatter::Serialize(FText& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(FWeakObjectPtr& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(FSoftObjectPtr& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(FSoftObjectPath& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(FLazyObjectPtr& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(FObjectPtr& Value)
{
	Inner << Value;
}

inline void FBinaryArchiveFormatter::Serialize(TArray<uint8>& Data)
{
	Inner << Data;
}

inline void FBinaryArchiveFormatter::Serialize(void* Data, uint64 DataSize)
{
	Inner.Serialize(Data, DataSize);
}
