// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/LockFreeList.h"
#include "CoreTypes.h"

#include <atomic>

template <class T, int TPaddingForCacheContention> class TLockFreePointerListUnordered;

/**
* Data storage for the large memory reader and writer.
*/

class FLargeMemoryData
{
public:

	CORE_API explicit FLargeMemoryData(const int64 PreAllocateBytes = 0);
	CORE_API ~FLargeMemoryData();

	/** Write data at the given offset. Returns true if the data was written. */
	CORE_API bool Write(void* InData, int64 InOffset, int64 InNum);

	/** Append data at the given offset. */
	UE_FORCEINLINE_HINT void Append(void* InData, int64 InNum)
	{
		Write(InData, NumBytes, InNum);
	}

	/** Read data at the given offset. Returns true if the data was read. */
	CORE_API bool Read(void* OutData, int64 InOffset, int64 InNum) const;

	/** Gets the size of the data written. */
	UE_FORCEINLINE_HINT int64 GetSize() const
	{
		return NumBytes;
	}

	/** Returns the written data.  */
	UE_FORCEINLINE_HINT uint8* GetData()
	{
		return Data;
	}

	/** Returns the written data.  */
	UE_FORCEINLINE_HINT const uint8* GetData() const
	{
		return Data;
	}

	/** Releases ownership of the written data. */
	CORE_API uint8* ReleaseOwnership();

	/** Check whether data is allocated or if the ownership was released. */
	bool HasData() const 
	{
		return Data != nullptr;
	}

	CORE_API void Reserve(int64 Size);

private:

	/** Non-copyable */
	FLargeMemoryData(const FLargeMemoryData&) = delete;
	FLargeMemoryData& operator=(const FLargeMemoryData&) = delete;

	/** Memory owned by this archive. Ownership can be released by calling ReleaseOwnership() */
	uint8* Data;

	/** Number of bytes currently written to our data buffer */
	int64 NumBytes;

	/** Number of bytes currently allocated for our data buffer */
	int64 MaxBytes;

	/** Resizes the data buffer to at least NumBytes with some slack */
	CORE_API void GrowBuffer();
};

/**
* Pooled storage of FLargeMemoryData instances, allowing allocation-free and lock-free access.
*/
class FPooledLargeMemoryData
{
public:
	CORE_API FPooledLargeMemoryData();
	CORE_API ~FPooledLargeMemoryData();
	FLargeMemoryData& Get() { return *Data; }
private:
	static void Free(FLargeMemoryData* Data);
	FLargeMemoryData* Data;
	static CORE_API TLockFreePointerListUnordered<FLargeMemoryData, 0> FreeList;
	static CORE_API std::atomic<int32> FreeListLength;
};
