// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"
#include "Serialization/MemoryArchive.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

enum class ELargeMemoryReaderFlags : uint8
{
	None =				0x0,	// No Flags
	TakeOwnership =		0x1,	// Archive will take ownership of the passed-in memory and free it on destruction
	Persistent =		0x2,	// Archive will be set as persistent when constructed
};

ENUM_CLASS_FLAGS(ELargeMemoryReaderFlags);

/**
* Archive for reading a large amount of arbitrary data from memory
*/
class FLargeMemoryReader : public FMemoryArchive
{
public:

	CORE_API FLargeMemoryReader(const uint8* InData, const int64 Num, ELargeMemoryReaderFlags InFlags = ELargeMemoryReaderFlags::None, const FName InArchiveName = NAME_None);

	CORE_API virtual void Serialize(void* OutData, int64 Num) override;

	/**
	* Gets the total size of the data buffer
	*/
	CORE_API virtual int64 TotalSize() override;

	/**
	* Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	* is in when a loading error occurs.
	*
	* This is overridden for the specific Archive Types
	**/
	CORE_API virtual FString GetArchiveName() const override;

	/**
	* Destructor
	*/
	CORE_API ~FLargeMemoryReader();

private:

	/** Non-copyable */
	FLargeMemoryReader(const FLargeMemoryReader&) = delete;
	FLargeMemoryReader& operator=(const FLargeMemoryReader&) = delete;

	/** Whether the data buffer should be freed when this archive is closed */
	const uint8 bFreeOnClose : 1;
	
	/** Data buffer we are reading from */
	const uint8* Data;

	/** Total size of the data buffer */
	const int64 NumBytes;

	/** Name of the archive this buffer is for */
	const FName ArchiveName;
};

