// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Memory/MemoryFwd.h"
#include "Memory/MemoryView.h"
#include "Serialization/LargeMemoryData.h"
#include "Serialization/MemoryArchive.h"
#include "UObject/NameTypes.h"

/**
* Archive for storing a large amount of arbitrary data to memory
*/
class FLargeMemoryWriter : public FMemoryArchive
{
public:
	
	CORE_API FLargeMemoryWriter(const int64 PreAllocateBytes = 0, bool bIsPersistent = false, const TCHAR* InFilename = nullptr);
	CORE_API virtual ~FLargeMemoryWriter();

	CORE_API virtual void Serialize(void* InData, int64 Num) override;

	/**
	* Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	* is in when a loading error occurs.
	*
	* This is overridden for the specific Archive Types
	**/
	CORE_API virtual FString GetArchiveName() const override;

	/**
	 * Gets the total size of the data written
	 */
	virtual int64 TotalSize() override
	{
		return Data.GetSize();
	}
	
	/**
	 * Returns the written data. To release this archive's ownership of the data, call ReleaseOwnership()
	 */
	CORE_API uint8* GetData() const;

	/**
	 * Returns a view on the written data
	 * 
	 * The view does not own the memory, so you must make sure you keep the memory writer
	 * alive while you are using the returned view.
	 * 
	 */
	inline FMemoryView GetView() const { return MakeMemoryView(GetData(), Data.GetSize()); }

	/** 
	 * Releases ownership of the written data
	 *
	 * Also returns the pointer, so that the caller only needs to call this function to take control
	 * of the memory.
	 */
	UE_FORCEINLINE_HINT uint8* ReleaseOwnership()
	{
		return Data.ReleaseOwnership();
	}

	/**
	 * Reserves memory such that the writer can contain at least Size number of bytes.
	 */
	void Reserve(int64 Size)
	{
		Data.Reserve(Size);
	}

private:

	FLargeMemoryData Data;

	/** Non-copyable */
	FLargeMemoryWriter(const FLargeMemoryWriter&) = delete;
	FLargeMemoryWriter& operator=(const FLargeMemoryWriter&) = delete;


	/** Archive name, used for debugging, by default set to NAME_None. */
	const FString ArchiveName;
};
