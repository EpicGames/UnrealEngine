// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Build.h"
#include "ProfilingDebugging/FormatArgsTrace.h"
#include "Trace/Config.h"
#include "Trace/Trace.h"

namespace UE { namespace Trace { class FChannel; } }

#if !defined(LOADTIMEPROFILERTRACE_ENABLED)
#if UE_TRACE_ENABLED && !UE_BUILD_SHIPPING
#define LOADTIMEPROFILERTRACE_ENABLED 1
#else
#define LOADTIMEPROFILERTRACE_ENABLED 0
#endif
#endif

#if LOADTIMEPROFILERTRACE_ENABLED

UE_TRACE_CHANNEL_EXTERN(LoadTimeChannel, CORE_API);
UE_TRACE_CHANNEL_EXTERN(AssetLoadTimeChannel, CORE_API);

struct FLoadTimeProfilerTrace
{
	struct FRequestGroupScope
	{
		template <typename... Types>
		FRequestGroupScope(const TCHAR* InFormatString, Types... FormatArgs)
		{
			FormatString = InFormatString;
			FormatArgsSize = FFormatArgsTrace::EncodeArguments(FormatArgsBuffer, FormatArgs...);
			OutputBegin();
		}

		CORE_API ~FRequestGroupScope();

	private:
		CORE_API void OutputBegin();

		const TCHAR* FormatString = nullptr;
		uint16 FormatArgsSize = 0;
		uint8 FormatArgsBuffer[1024];
	};
};

#define TRACE_LOADTIME_REQUEST_GROUP_SCOPE(Format, ...) \
	FLoadTimeProfilerTrace::FRequestGroupScope __LoadTimeTraceRequestGroupScope(Format, ##__VA_ARGS__);

#else
#define TRACE_LOADTIME_REQUEST_GROUP_SCOPE(...)
#endif
