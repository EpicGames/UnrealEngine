// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameBatchSerialization.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FArchive;

/**
 * Index and name number into a name map.
 */
class FMappedName
{
	static constexpr uint32 InvalidIndex = ~uint32(0);
	static constexpr uint32 IndexBits = 30u;
	static constexpr uint32 IndexMask = (1u << IndexBits) - 1u;
	static constexpr uint32 TypeMask = ~IndexMask;
	static constexpr uint32 TypeShift = IndexBits;

public:
	enum class EType
	{
		Package,
		Container,
		Global
	};

	inline FMappedName() = default;

	static inline FMappedName Create(const uint32 InIndex, const uint32 InNumber, EType InType)
	{
		check(InIndex < (1 << TypeShift));
		return FMappedName((uint32(InType) << TypeShift) | InIndex, InNumber);
	}

	static inline FMappedName FromMinimalName(const FMinimalName& MinimalName)
	{
		return *reinterpret_cast<const FMappedName*>(&MinimalName);
	}

	static inline bool IsResolvedToMinimalName(const FMinimalName& MinimalName)
	{
		// Not completely safe, relies on that no FName will have its Index and Number equal to Max_uint32
		const FMappedName MappedName = FromMinimalName(MinimalName);
		return MappedName.IsValid();
	}

	static inline FName SafeMinimalNameToName(const FMinimalName& MinimalName)
	{
		return IsResolvedToMinimalName(MinimalName) ? MinimalNameToName(MinimalName) : NAME_None;
	}

	inline FMinimalName ToUnresolvedMinimalName() const
	{
		return *reinterpret_cast<const FMinimalName*>(this);
	}

	inline bool IsValid() const
	{
		return Index != InvalidIndex && Number != InvalidIndex;
	}

	inline EType GetType() const
	{
		return static_cast<EType>(uint32((Index & TypeMask) >> TypeShift));
	}

	inline bool IsGlobal() const
	{
		return ((Index & TypeMask) >> TypeShift) != 0;
	}

	inline uint32 GetIndex() const
	{
		return Index & IndexMask;
	}

	inline uint32 GetNumber() const
	{
		return Number;
	}

	inline bool operator!=(FMappedName Other) const
	{
		return Index != Other.Index || Number != Other.Number;
	}

	FName ResolveName(TConstArrayView<FDisplayNameEntryId> Names) const
	{
		return Names[GetIndex()].ToName(GetNumber());
	}

	CORE_API friend FArchive& operator<<(FArchive& Ar, FMappedName& MappedName);

private:
	inline FMappedName(const uint32 InIndex, const uint32 InNumber)
		: Index(InIndex)
		, Number(InNumber) { }

	uint32 Index = InvalidIndex;
	uint32 Number = InvalidIndex;
};

/*
 * Maps serialized name entries to names.
 */
class FNameMap
{
public:
	inline int32 Num() const
	{
		return NameEntries.Num();
	}

	CORE_API void Load(FArchive& Ar, FMappedName::EType NameMapType);

	FName GetName(const FMappedName& MappedName) const
	{
		check(MappedName.GetType() == NameMapType);
		check(MappedName.GetIndex() < uint32(NameEntries.Num()));

		return NameEntries[MappedName.GetIndex()].ToName(MappedName.GetNumber());
	}

	bool TryGetName(const FMappedName& MappedName, FName& OutName) const
	{
		check(MappedName.GetType() == NameMapType);

		uint32 Index = MappedName.GetIndex();
		if (Index < uint32(NameEntries.Num()))
		{
			OutName = NameEntries[MappedName.GetIndex()].ToName(MappedName.GetNumber());

			return true;
		}

		return false;
	}

	FMinimalName GetMinimalName(const FMappedName& MappedName) const
	{
		// Wasteful, looks up display name then discards it
		return NameToMinimalName(GetName(MappedName));
	}

	using RangedForConstIteratorType = TArray<FDisplayNameEntryId>::RangedForConstIteratorType;
	RangedForConstIteratorType begin() const { return NameEntries.begin(); }
	RangedForConstIteratorType end() const { return NameEntries.end(); }

private:
	TArray<FDisplayNameEntryId> NameEntries;
	FMappedName::EType NameMapType = FMappedName::EType::Global;
};
