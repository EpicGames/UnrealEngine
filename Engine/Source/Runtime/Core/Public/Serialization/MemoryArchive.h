// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"

/**
 * Base class for serializing arbitrary data in memory.
 */
class FMemoryArchive : public FArchive
{
public:
	/**
  	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const { return TEXT("FMemoryArchive"); }

	void Seek( int64 InPos ) final
	{
		Offset = InPos;
	}
	int64 Tell() final
	{
		return Offset;
	}

	using FArchive::operator<<; // For visibility of the overloads we don't override

	virtual FArchive& operator<<( class FName& N ) override
	{
		// Serialize the FName as a string
		if (IsLoading())
		{
			FString StringName;
			*this << StringName;
			N = FName(*StringName);
		}
		else
		{
			FString StringName = N.ToString();
			*this << StringName;
		}
		return *this;
	}

	virtual FArchive& operator<<( class UObject*& Res ) override
	{
		// Not supported through this archive
		check(0);
		return *this;
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual FArchive& operator<<( Verse::VCell*& Res ) override
	{
		// Not supported through this archive
		check(0);
		return *this;
	}
#endif

protected:

	/** Marked as protected to avoid instantiating this class directly */
	FMemoryArchive()
		: FArchive(), Offset(0)
	{
	}

	int64				Offset;
};

