// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Math/NumericLimits.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "Serialization/MemoryArchive.h"
#include "Misc/SecureHash.h"
#include "Hash/Blake3.h"
#include "Serialization/ArchiveProxy.h"

/**
 * Archive for hashing arbitrary data
 */
template <typename HashBuilder, typename HashDigest>
class TMemoryHasher : public FMemoryArchive
{
public:
	TMemoryHasher()
	: FMemoryArchive()
	{
		this->SetIsSaving(true);
		this->SetIsPersistent(false);
	}

	virtual void Serialize(void* Data, int64 Num) override
	{
		BuilderState.Update(reinterpret_cast<uint8*>(Data), Num);
	}

	HashDigest Finalize()
	{
		return BuilderState.Finalize();
	}

	/**
  	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const override { return TEXT("TMemoryHasherTemplate"); }

	int64 TotalSize() override
	{
		return 0;
	}

protected:

	/** Context data needed to build the hash */
	HashBuilder		BuilderState;
};

using FMemoryHasherSHA1 = TMemoryHasher<FSHA1, FSHAHash>;
using FMemoryHasherBlake3 = TMemoryHasher<FBlake3, FBlake3Hash>;

/**
 * Archive for hashing another archive
 */
namespace UE
{

template <typename HashBuilderType>
class THashingArchiveProxy : public FArchiveProxy
{
public:
	using FArchiveProxy::FArchiveProxy;

	void Serialize(void* V, int64 Length) override
	{
		FArchiveProxy::Serialize(V, Length);
		HashBuilder.Update(V, uint64(Length));
	}

	void Seek(int64 InPos) override
	{
		checkf(InPos == Tell(), TEXT("A hash cannot be computed when serialization relies on seeking."));
		FArchiveProxy::Seek(InPos);
	}

	auto GetHash() -> decltype(DeclVal<HashBuilderType>().Finalize())
	{
		return HashBuilder.Finalize();
	}

private:
	HashBuilderType HashBuilder;
};

} // UE
