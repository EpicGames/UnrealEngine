// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/HashTable.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/MemoryBase.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformString.h"
#include "HAL/PreprocessorHelpers.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Crc.h"
#include "Misc/SecureHash.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryImageWriter.h"
#include "Serialization/MemoryLayout.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/EnableIf.h"
#include "Templates/RefCounting.h"
#include "Templates/TypeHash.h"
#include "Traits/IsCharType.h"
#include "UObject/NameTypes.h"

class FCbFieldView;
class FCbWriter;
template <typename T> struct TIsContiguousContainer;

#if defined(WITH_RTTI) || defined(_CPPRTTI) || defined(__GXX_RTTI) || WITH_EDITOR
#include <typeinfo>
#endif

class FMemoryImage;
class FMemoryImageString;

// Store type dependencies for loaded frozen memory images
// This uses a bit of memory and adds small CPU cost to loading, but is required to support unfreezing memory images that target platforms other than the current
// This is also required for creating frozen memory images
#define UE_MEMORYIMAGE_TRACK_TYPE_DEPENDENCIES (WITH_EDITORONLY_DATA)

class FPointerTableBase
{
public:
	virtual ~FPointerTableBase() = default;
	virtual int32 AddIndexedPointer(const FTypeLayoutDesc& TypeDesc, void* Ptr) = 0;
	virtual void* GetIndexedPointer(const FTypeLayoutDesc& TypeDesc, uint32 i) const = 0;

	CORE_API virtual void SaveToArchive(FArchive& Ar, const FPlatformTypeLayoutParameters& LayoutParams, const void* FrozenObject) const;
	CORE_API virtual bool LoadFromArchive(FArchive& Ar, const FPlatformTypeLayoutParameters& LayoutParams, void* FrozenObject);

#if UE_MEMORYIMAGE_TRACK_TYPE_DEPENDENCIES
	CORE_API int32 AddTypeDependency(const FTypeLayoutDesc& TypeDesc);
	inline const FTypeLayoutDesc* GetTypeDependency(int32 Index) const { return TypeDependencies[Index]; }
private:
	TArray<const FTypeLayoutDesc*> TypeDependencies;
#else
	int32 AddTypeDependency(const FTypeLayoutDesc& TypeDesc) { return INDEX_NONE; }
	inline const FTypeLayoutDesc* GetTypeDependency(int32 Index) const { return nullptr; }
#endif
};

template<typename T>
struct TMemoryImageObject
{
	TMemoryImageObject() : TypeDesc(nullptr), Object(nullptr), FrozenSize(0u) {}

	TMemoryImageObject(const FTypeLayoutDesc& InTypeDesc, T* InObject, uint32 InFrozenSize)
		: TypeDesc(&InTypeDesc)
		, Object(InObject)
		, FrozenSize(InFrozenSize)
	{}

	template<typename TOther>
	TMemoryImageObject(TOther* InObject)
		: TypeDesc(InObject ? &GetTypeLayoutDesc(nullptr, *InObject) : nullptr)
		, Object(InObject)
		, FrozenSize(0u)
	{}

	template<typename TOther>
	explicit TMemoryImageObject(const TMemoryImageObject<TOther>& Rhs)
		: TypeDesc(Rhs.TypeDesc)
		, Object(static_cast<T*>(Rhs.Object))
		, FrozenSize(Rhs.FrozenSize)
	{}

	void Destroy(const FPointerTableBase* PointerTable);

	// Returns true if the frozen/unfrozen state of the object was changed
	bool Freeze(FPointerTableBase* PointerTable);
	bool Unfreeze(const FPointerTableBase* PointerTable);

	const FTypeLayoutDesc* TypeDesc;
	T* Object;
	uint32 FrozenSize;
};
using FMemoryImageObject = TMemoryImageObject<void>;

CORE_API FMemoryImageObject FreezeMemoryImageObject(const void* Object, const FTypeLayoutDesc& TypeDesc, FPointerTableBase* PointerTable);
CORE_API void* UnfreezeMemoryImageObject(const void* FrozenObject, const FTypeLayoutDesc& TypeDesc, const FPointerTableBase* PointerTable);

template<typename T>
inline void TMemoryImageObject<T>::Destroy(const FPointerTableBase* PointerTable)
{
	if (Object)
	{
		InternalDeleteObjectFromLayout(Object, *TypeDesc, PointerTable, FrozenSize > 0u);
		if (FrozenSize > 0u)
		{
			// InternalDeleteObjectFromLayout will delete unfrozen objects,
			// but won't free frozen objects, since that's not safe for internal object pointers
			// Here we are working with a root-level object, so it's safe to free it
			FMemory::Free(Object);
		}
		Object = nullptr;
		FrozenSize = 0u;
	}
}


template<typename T>
inline bool TMemoryImageObject<T>::Freeze(FPointerTableBase* PointerTable)
{
	if (FrozenSize == 0u && Object)
	{
		const FMemoryImageObject FrozenContent = FreezeMemoryImageObject(Object, *TypeDesc, PointerTable);
		Destroy(nullptr);
		Object = static_cast<T*>(FrozenContent.Object);
		FrozenSize = FrozenContent.FrozenSize;
		return true;
	}
	return false;
}

template<typename T>
inline bool TMemoryImageObject<T>::Unfreeze(const FPointerTableBase* PointerTable)
{
	if (FrozenSize > 0u)
	{
		void* UnfrozenObject = UnfreezeMemoryImageObject(Object, *TypeDesc, PointerTable);
		Destroy(PointerTable);
		Object = static_cast<T*>(UnfrozenObject);
		FrozenSize = 0u;
		return true;
	}
	return false;
}

struct FMemoryImageVTablePointer
{
	uint64 TypeNameHash;
	uint32 VTableOffset;
	uint32 Offset;

	inline bool operator==(const FMemoryImageVTablePointer& Rhs) const
	{
		return TypeNameHash == Rhs.TypeNameHash && VTableOffset == Rhs.VTableOffset && Offset == Rhs.Offset;
	}
	inline bool operator!=(const FMemoryImageVTablePointer& Rhs) const
	{
		return !(*this == Rhs);
	}
	inline bool operator<(const FMemoryImageVTablePointer& Rhs) const
	{
		if (TypeNameHash != Rhs.TypeNameHash) return TypeNameHash < Rhs.TypeNameHash;
		if (VTableOffset != Rhs.VTableOffset) return VTableOffset < Rhs.VTableOffset;
		return Offset < Rhs.Offset;
	}
};

struct FMemoryImageNamePointer
{
	FName Name;
	uint32 Offset;

	inline bool operator==(const FMemoryImageNamePointer& Rhs) const
	{
		return Offset == Rhs.Offset && Name.Compare(Rhs.Name) == 0;
	}
	inline bool operator!=(const FMemoryImageNamePointer& Rhs) const
	{
		return !(*this == Rhs);
	}
	inline bool operator<(const FMemoryImageNamePointer& Rhs) const
	{
		if (Name != Rhs.Name)
		{
			return Name.LexicalLess(Rhs.Name);
		}
		return Offset < Rhs.Offset;
	}
};

struct FMemoryImageResult
{
	TArray<uint8> Bytes;
	FPointerTableBase* PointerTable = nullptr;
	FPlatformTypeLayoutParameters TargetLayoutParameters;
	TArray<FMemoryImageVTablePointer> VTables;
	TArray<FMemoryImageNamePointer> ScriptNames;
	TArray<FMemoryImageNamePointer> MemoryImageNames;

	CORE_API void SaveToArchive(FArchive& Ar) const;
	CORE_API bool ApplyPatches(void* FrozenObject, uint64 FrozenObjectSize) const;
	CORE_API static FMemoryImageObject LoadFromArchive(FArchive& Ar, const FTypeLayoutDesc& TypeDesc, FPointerTableBase* PointerTable, FPlatformTypeLayoutParameters& OutLayoutParameters);
};

class FMemoryImageSection : public FRefCountedObject
{
public:
	struct FSectionPointer
	{
		uint32 SectionIndex;
		uint32 PointerOffset;
		uint32 Offset;
	};

	FMemoryImageSection(FMemoryImage* InImage)
		: ParentImage(InImage)
		, MaxAlignment(1u)
	{}

	uint32 GetOffset() const { return Bytes.Num(); }

	uint32 WriteAlignment(uint32 Alignment)
	{
		const uint32 PrevSize = Bytes.Num();
		const uint32 Offset = Align(PrevSize, Alignment);
		Bytes.SetNumZeroed(Offset);
		MaxAlignment = FMath::Max(MaxAlignment, Alignment);
		return Offset;
	}

	void WritePaddingToSize(uint32 Offset)
	{
		check(Offset >= (uint32)Bytes.Num());
		Bytes.SetNumZeroed(Offset);
	}

	uint32 WriteBytes(const void* Data, uint32 Size)
	{
		const uint32 Offset = GetOffset();
		Bytes.SetNumUninitialized(Offset + Size);
		FMemory::Memcpy(Bytes.GetData() + Offset, Data, Size);
		return Offset;
	}

	uint32 WriteZeroBytes(int32 Num)
	{
		const uint32 Offset = GetOffset();
		Bytes.SetNumZeroed(Offset + Num);
		return Offset;
	}

	template<typename T>
	uint32 WriteBytes(const T& Data) { return WriteBytes(&Data, sizeof(T)); }

	CORE_API FMemoryImageSection* WritePointer(const FTypeLayoutDesc& StaticTypeDesc, const FTypeLayoutDesc& DerivedTypeDesc, uint32* OutOffsetToBase = nullptr);
	CORE_API uint32 WriteRawPointerSizedBytes(uint64 PointerValue);
	CORE_API uint32 WriteVTable(const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& DerivedTypeDesc);
	CORE_API uint32 WriteFMemoryImageName(int32 NumBytes, const FName& Name);
	CORE_API uint32 WriteFScriptName(const FScriptName& Name);
	CORE_API uint32 Flatten(FMemoryImageResult& OutResult) const;

	CORE_API void ComputeHash();

	FMemoryImage* ParentImage;
	TArray<uint8> Bytes;
	TArray<FSectionPointer> Pointers;
	TArray<FMemoryImageVTablePointer> VTables;
	TArray<FMemoryImageNamePointer> ScriptNames;
	TArray<FMemoryImageNamePointer> MemoryImageNames;
	FSHAHash Hash;
	uint32 MaxAlignment;
};

class FMemoryImage
{
public:
	FMemoryImage()
		: PointerTable(nullptr)
		, PrevPointerTable(nullptr)
		, CurrentStruct(nullptr)
	{
		HostLayoutParameters.InitializeForCurrent();
	}

	FPointerTableBase& GetPointerTable() const { check(PointerTable); return *PointerTable; }
	const FPointerTableBase& GetPrevPointerTable() const { check(PrevPointerTable); return *PrevPointerTable; }

	FMemoryImageSection* AllocateSection()
	{
		FMemoryImageSection* Section = new FMemoryImageSection(this);
		// reserving memory here could reduce the reallocations, but leads to huge spikes for images with many sections. TODO: try chunked array or a better heuristic value for reservation
		Sections.Add(Section);
		return Section;
	}

	/** Merging duplicate sections will make the resulting memory image smaller.
	 * This will only work for data that is expected to be read-only after freezing.  Merging sections will break any manual fix-ups applied to the frozen data
	 */
	CORE_API void Flatten(FMemoryImageResult& OutResult, bool bMergeDuplicateSections = false);

	TArray<TRefCountPtr<FMemoryImageSection>> Sections;
	FPointerTableBase* PointerTable;
	const FPointerTableBase* PrevPointerTable;
	FPlatformTypeLayoutParameters HostLayoutParameters;
	FPlatformTypeLayoutParameters TargetLayoutParameters;
	const class UStruct* CurrentStruct;
};

/**
 * Value of this struct should never be a valid unfrozen pointer (i.e. a memory address). We rely on real pointers to have lowest bit(s) 0 this days for the alignment, this is checked later.
 * Unfortunately, we cannot use bitfields as their layout might be compiler-specific, and the data for the target platform is being prepared with a different compiler during the cook.
 */
struct FFrozenMemoryImagePtr
{
	static constexpr uint64 bIsFrozenBits = 1;
	static constexpr uint64 OffsetBits = 40;
	static constexpr uint64 TypeIndexBits = 64 - OffsetBits - bIsFrozenBits;

	static constexpr uint64 bIsFrozenShift = 0;
	static constexpr uint64 TypeIndexShift = bIsFrozenBits;
	static constexpr uint64 OffsetShift = bIsFrozenBits + TypeIndexBits;

	static constexpr uint64 bIsFrozenMask = (1ULL << bIsFrozenShift);
	static constexpr uint64 TypeIndexMask = (((1ULL << TypeIndexBits) - 1ULL) << TypeIndexShift);
	static constexpr uint64 OffsetMask = (((1ULL << OffsetBits) - 1ULL) << OffsetShift);

	uint64 Packed;

	/** Whether the value is indeed a frozen pointer, must come first to avoid being set in regular pointers - which are expected to point at padded things so are never even. */
	bool IsFrozen() const
	{
		return (Packed & bIsFrozenMask) != 0;
	}

	void SetIsFrozen(bool bTrue)
	{
		Packed = (Packed & ~bIsFrozenMask) | (bTrue ? 1 : 0);
	}

	/** Signed offset from the current position in the memory image. */
	int64 GetOffsetFromThis() const
	{
		// Since the offset occupies the highest part of the int64, its sign is preserved.
		// Not masking as there's nothing to the left of the Offset
		static_assert(OffsetShift + OffsetBits == 64);
		return static_cast<int64>(Packed/* & OffsetMask*/) >> OffsetShift;
	}

	void SetOffsetFromThis(int64 Offset)
	{
		Packed = (Packed & ~OffsetMask) | (static_cast<uint64>(Offset << OffsetShift) & OffsetMask);
	}

	/** The pointer type index in the pointer table. Does not store other negative values except for INDEX_NONE */
	int32 GetTypeIndex() const
	{
		return static_cast<int32>((Packed & TypeIndexMask) >> TypeIndexShift) - 1;
	}

	void SetTypeIndex(int32 TypeIndex)
	{
		static_assert(INDEX_NONE == -1, "TypeIndex cannot store INDEX_NONE when it's not -1");
		// PVS warns about a possible overflow in TypeIndex + 1. We don't care as we don't expect 2^31 type indices anyway
		Packed = (Packed & ~TypeIndexMask) | ((static_cast<uint64>(TypeIndex + 1) << TypeIndexShift) & TypeIndexMask); //-V1028
	}
};

static_assert(sizeof(FFrozenMemoryImagePtr) == sizeof(uint64), "FFrozenMemoryImagePtr is larger than a native pointer would be");

template<typename T>
class TMemoryImagePtr
{
public:
	inline bool IsFrozen() const { return Frozen.IsFrozen(); }
	inline bool IsValid() const { return UnfrozenPtr != nullptr; }
	inline bool IsNull() const { return UnfrozenPtr == nullptr; }

	inline TMemoryImagePtr() : UnfrozenPtr(nullptr) {}
	inline TMemoryImagePtr(T* InPtr) : UnfrozenPtr(InPtr) { check(!Frozen.IsFrozen()); }
	inline TMemoryImagePtr(const TMemoryImagePtr<T>& InPtr) : UnfrozenPtr(InPtr.Get()) { check(!Frozen.IsFrozen()); }
	inline TMemoryImagePtr& operator=(T* InPtr) { UnfrozenPtr = InPtr; check(!Frozen.IsFrozen()); return *this; }
	inline TMemoryImagePtr& operator=(const TMemoryImagePtr<T>& InPtr) { UnfrozenPtr = InPtr.Get(); check(!Frozen.IsFrozen()); return *this; }

	inline ~TMemoryImagePtr() = default; 

	inline int64 GetFrozenOffsetFromThis() const { check(IsFrozen()); return Frozen.GetOffsetFromThis(); }
	inline int32 GetFrozenTypeIndex() const { check(IsFrozen()); return Frozen.GetTypeIndex(); }

	inline T* Get() const
	{
		return IsFrozen() ? GetFrozenPtrInternal() : UnfrozenPtr;
	}

	inline T* GetChecked() const { T* Value = Get(); check(Value); return Value; }
	inline T* operator->() const { return GetChecked(); }
	inline T& operator*() const { return *GetChecked(); }
	inline operator T*() const { return Get(); }

	void SafeDelete(const FPointerTableBase* PtrTable = nullptr)
	{
		T* RawPtr = Get();
		if (RawPtr)
		{
			DeleteObjectFromLayout(RawPtr, PtrTable, IsFrozen());
			UnfrozenPtr = nullptr;
		}
	}

	void WriteMemoryImageWithDerivedType(FMemoryImageWriter& Writer, const FTypeLayoutDesc* DerivedTypeDesc) const
	{
		const T* RawPtr = Get();
		if (RawPtr)
		{
			check(DerivedTypeDesc);
			uint32 OffsetToBase = 0u;
			FMemoryImageWriter PointerWriter = Writer.WritePointer(StaticGetTypeLayoutDesc<T>(), *DerivedTypeDesc, &OffsetToBase);
			PointerWriter.WriteObject((uint8*)RawPtr - OffsetToBase, *DerivedTypeDesc);
		}
		else
		{
			Writer.WriteNullPointer();
		}
	}

private:
	inline T* GetFrozenPtrInternal() const
	{
		return (T*)((char*)this + Frozen.GetOffsetFromThis());
	}

protected:
	union
	{
		uint64 Packed;
		FFrozenMemoryImagePtr Frozen;
		T* UnfrozenPtr;
	};
};

namespace Freeze
{
	template<typename T>
	void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const TMemoryImagePtr<T>& Object, const FTypeLayoutDesc&)
	{
		T* RawPtr = Object.Get();
		if (RawPtr)
		{
			const FTypeLayoutDesc& DerivedTypeDesc = GetTypeLayoutDesc(Writer.TryGetPrevPointerTable(), *RawPtr);

			uint32 OffsetToBase = 0u;
			FMemoryImageWriter PointerWriter = Writer.WritePointer(StaticGetTypeLayoutDesc<T>(), DerivedTypeDesc, &OffsetToBase);
			PointerWriter.WriteObject((uint8*)RawPtr - OffsetToBase, DerivedTypeDesc);
		}
		else
		{
			Writer.WriteNullPointer();
		}
	}

	template<typename T>
	uint32 IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const TMemoryImagePtr<T>& Object, void* OutDst)
	{
		const T* RawPtr = Object.Get();
		if (RawPtr)
		{
			// Compile-time type of the thing we're pointing to
			const FTypeLayoutDesc& StaticTypeDesc = StaticGetTypeLayoutDesc<T>();
			
			// Actual run-time type of the thing we're pointing to
			const FTypeLayoutDesc* DerivedTypeDesc = Context.GetDerivedTypeDesc(StaticTypeDesc, Object.GetFrozenTypeIndex());
			if (!DerivedTypeDesc && Context.bIsFrozenForCurrentPlatform)
			{
				// It's possible we won't be able to get derived type desc from the context, if we're not storing type dependencies
				// In this case, if we're unfreezing data for the current platform, we can just grab the derived type directly from the frozen object
				// If we're NOT unfreezing data for current platform, we can't access the frozen object, so we'll fail in that case
				DerivedTypeDesc = &GetTypeLayoutDesc(Context.PrevPointerTable, *RawPtr);
			}
			if (DerivedTypeDesc)
			{
				// 'this' offset to adjust from the compile-time type to the run-time type
				const uint32 OffsetToBase = DerivedTypeDesc->GetOffsetToBase(StaticTypeDesc);

				void* UnfrozenMemory = ::operator new(DerivedTypeDesc->Size);
				Context.UnfreezeObject((uint8*)RawPtr - OffsetToBase, *DerivedTypeDesc, UnfrozenMemory);
				T* UnfrozenObject = (T*)((uint8*)UnfrozenMemory + OffsetToBase);
				new(OutDst) TMemoryImagePtr<T>(UnfrozenObject);
			}
			else
			{
				new(OutDst) TMemoryImagePtr<T>(nullptr);
			}
		}
		else
		{
			new(OutDst) TMemoryImagePtr<T>(nullptr);
		}
		return sizeof(Object);
	}

	template<typename T>
	uint32 IntrinsicAppendHash(const TMemoryImagePtr<T>* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		return AppendHashForNameAndSize(TypeDesc.Name, sizeof(TMemoryImagePtr<T>), Hasher);
	}

	template<typename T>
	inline uint32 IntrinsicGetTargetAlignment(const TMemoryImagePtr<T>* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams)
	{
		return FMath::Min(8u, LayoutParams.MaxFieldAlignment);
	}

	template<typename T>
	void IntrinsicToString(const TMemoryImagePtr<T>& Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
	{
		T* RawPtr = Object.Get();
		if (RawPtr)
		{
			// Compile-time type of the thing we're pointing to
			const FTypeLayoutDesc& StaticTypeDesc = StaticGetTypeLayoutDesc<T>();
			// Actual run-time type of the thing we're pointing to
			const FTypeLayoutDesc& DerivedTypeDesc = GetTypeLayoutDesc(OutContext.TryGetPrevPointerTable(), *RawPtr);
			// 'this' offset to adjust from the compile-time type to the run-time type
			const uint32 OffsetToBase = DerivedTypeDesc.GetOffsetToBase(StaticTypeDesc);

			if (Object.IsFrozen())
			{
				OutContext.AppendFrozenPointer(StaticTypeDesc, Object.GetFrozenTypeIndex());
			}
			else
			{
				OutContext.AppendUnfrozenPointer(StaticTypeDesc);
			}
			DerivedTypeDesc.ToStringFunc((uint8*)RawPtr - OffsetToBase, DerivedTypeDesc, LayoutParams, OutContext);
		}
		else
		{
			OutContext.AppendNullptr();
		}
	}
}

DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT(template<typename T>, TMemoryImagePtr<T>);

template<typename T>
class TUniqueMemoryImagePtr : public TMemoryImagePtr<T>
{
public:
	inline TUniqueMemoryImagePtr()
		: TMemoryImagePtr<T>(nullptr)
	{}
	explicit inline TUniqueMemoryImagePtr(T* InPtr)
		: TMemoryImagePtr<T>(InPtr)
	{ }

	inline TUniqueMemoryImagePtr(TUniqueMemoryImagePtr&& Other)
	{
		this->SafeDelete();
		this->Ptr = Other.Ptr;
		Other.Ptr = nullptr;
	}
	inline ~TUniqueMemoryImagePtr()
	{
		this->SafeDelete();
	}
	inline TUniqueMemoryImagePtr& operator=(T* InPtr)
	{
		this->SafeDelete();
		this->Ptr = InPtr;
		return *this;
	}
	inline TUniqueMemoryImagePtr& operator=(TUniqueMemoryImagePtr&& Other)
	{
		if (this != &Other)
		{
			// We _should_ delete last like TUniquePtr, but we have issues with SafeDelete, and being Frozen or not
			this->SafeDelete();
			this->Ptr = Other.Ptr;
			Other.Ptr = nullptr;
		}

		return *this;
	}

};

class FMemoryImageAllocatorBase
{
	UE_NONCOPYABLE(FMemoryImageAllocatorBase);
public:
	FMemoryImageAllocatorBase() = default;

	/**
	* Moves the state of another allocator into this one.
	* Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
	* @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
	*/
	CORE_API void MoveToEmpty(FMemoryImageAllocatorBase& Other);

	/** Destructor. */
	CORE_API ~FMemoryImageAllocatorBase();

	// FContainerAllocatorInterface
	UE_FORCEINLINE_HINT FScriptContainerElement* GetAllocation() const
	{
		return Data.Get();
	}

	UE_FORCEINLINE_HINT SIZE_T GetAllocatedSize(int32 CurrentMax, SIZE_T NumBytesPerElement) const
	{
		return CurrentMax * NumBytesPerElement;
	}
	UE_FORCEINLINE_HINT bool HasAllocation()
	{
		return Data.IsValid();
	}
	UE_FORCEINLINE_HINT int64 GetFrozenOffsetFromThis() const { return Data.GetFrozenOffsetFromThis(); }

	CORE_API void ResizeAllocation(int32 PreviousNumElements, int32 NumElements, SIZE_T NumBytesPerElement, uint32 Alignment);
	CORE_API void WriteMemoryImage(FMemoryImageWriter& Writer, const FTypeLayoutDesc& TypeDesc, int32 NumAllocatedElements, uint32 Alignment) const;
	CORE_API void ToString(const FTypeLayoutDesc& TypeDesc, int32 NumAllocatedElements, int32 MaxAllocatedElements, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext) const;
	CORE_API void CopyUnfrozen(const FMemoryUnfreezeContent& Context, const FTypeLayoutDesc& TypeDesc, int32 NumAllocatedElements, void* Dst) const;

private:
	TMemoryImagePtr<FScriptContainerElement> Data;
};

template<uint32 Alignment = DEFAULT_ALIGNMENT>
class TMemoryImageAllocator
{
public:
	using SizeType = int32;

	enum { NeedsElementType = false };
	enum { RequireRangeCheck = true };
	enum { SupportsFreezeMemoryImage = true };

	class ForAnyElementType : public FMemoryImageAllocatorBase
	{
	public:
		/** Default constructor. */
		ForAnyElementType() = default;

		UE_FORCEINLINE_HINT SizeType GetInitialCapacity() const
		{
			return 0;
		}
		UE_FORCEINLINE_HINT int32 CalculateSlackReserve(int32 NewMax, int32 NumBytesPerElement) const
		{
			return DefaultCalculateSlackReserve(NewMax, NumBytesPerElement, true, Alignment);
		}
		UE_FORCEINLINE_HINT int32 CalculateSlackReserve(int32 NewMax, int32 NumBytesPerElement, uint32 AlignmentOfElement) const
		{
			return DefaultCalculateSlackReserve(NewMax, NumBytesPerElement, true, AlignmentOfElement);
		}
		UE_FORCEINLINE_HINT int32 CalculateSlackShrink(int32 NewMax, int32 CurrentMax, int32 NumBytesPerElement) const
		{
			return DefaultCalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, true, Alignment);
		}
		UE_FORCEINLINE_HINT int32 CalculateSlackShrink(int32 NewMax, int32 CurrentMax, int32 NumBytesPerElement, uint32 AlignmentOfElement) const
		{
			return DefaultCalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, true, AlignmentOfElement);
		}
		UE_FORCEINLINE_HINT int32 CalculateSlackGrow(int32 NewMax, int32 CurrentMax, int32 NumBytesPerElement) const
		{
			return DefaultCalculateSlackGrow(NewMax, CurrentMax, NumBytesPerElement, true, Alignment);
		}
		UE_FORCEINLINE_HINT int32 CalculateSlackGrow(int32 NewMax, int32 CurrentMax, int32 NumBytesPerElement, uint32 AlignmentOfElement) const
		{
			return DefaultCalculateSlackGrow(NewMax, CurrentMax, NumBytesPerElement, true, AlignmentOfElement);
		}
		UE_FORCEINLINE_HINT void ResizeAllocation(int32 CurrentNum, int32 NewMax, SIZE_T NumBytesPerElement)
		{
			FMemoryImageAllocatorBase::ResizeAllocation(CurrentNum, NewMax, NumBytesPerElement, Alignment);
		}
		UE_FORCEINLINE_HINT void ResizeAllocation(int32 CurrentNum, int32 NewMax, SIZE_T NumBytesPerElement, uint32 AlignmentOfElement)
		{
			FMemoryImageAllocatorBase::ResizeAllocation(CurrentNum, NewMax, NumBytesPerElement, AlignmentOfElement);
		}

		UE_FORCEINLINE_HINT void WriteMemoryImage(FMemoryImageWriter& Writer, const FTypeLayoutDesc& TypeDesc, int32 NumAllocatedElements) const
		{
			FMemoryImageAllocatorBase::WriteMemoryImage(Writer, TypeDesc, NumAllocatedElements, Alignment);
		}
	};

	template<typename ElementType>
	class ForElementType : public ForAnyElementType
	{
	public:
		ForElementType() {}
		UE_FORCEINLINE_HINT ElementType* GetAllocation() const { return (ElementType*)ForAnyElementType::GetAllocation(); }
	};
};

//@todo stever
/*static_assert(
	sizeof(TMemoryImageAllocator<>::ForAnyElementType) == sizeof(FDefaultAllocator::ForAnyElementType) && alignof(TMemoryImageAllocator<>::ForAnyElementType) == alignof(FDefaultAllocator::ForAnyElementType),
	"TMemoryImageAllocator must be the same layout as FDefaultAllocator for our FScriptArray hacks to work"
);*/

template <uint32 Alignment>
struct TAllocatorTraits<TMemoryImageAllocator<Alignment>> : TAllocatorTraitsBase<TMemoryImageAllocator<Alignment>>
{
	enum { IsZeroConstruct = true };
	enum { SupportsFreezeMemoryImage = true };
	enum { SupportsElementAlignment = true };
};

using FMemoryImageAllocator = TMemoryImageAllocator<>;

using FMemoryImageSparseArrayAllocator = TSparseArrayAllocator<FMemoryImageAllocator, FMemoryImageAllocator>;
using FMemoryImageSetAllocator = TSetAllocator<FMemoryImageSparseArrayAllocator, FMemoryImageAllocator>;

template<typename T>
using TMemoryImageArray = TArray<T, FMemoryImageAllocator>;

template<typename ElementType, typename KeyFuncs = DefaultKeyFuncs<ElementType>>
using TMemoryImageSet = TSet<ElementType, KeyFuncs, FMemoryImageSetAllocator>;

template <typename KeyType, typename ValueType, typename KeyFuncs = TDefaultMapHashableKeyFuncs<KeyType, ValueType, false>>
using TMemoryImageMap = TMap<KeyType, ValueType, FMemoryImageSetAllocator, KeyFuncs>;

template <>
struct TIsContiguousContainer<FMemoryImageString>
{
	static constexpr bool Value = true;
};

class FMemoryImageString
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FMemoryImageString, CORE_API, NonVirtual);
private:
	/** Array holding the character data */
	using DataType = TMemoryImageArray<TCHAR>;
	LAYOUT_FIELD(DataType, Data);

	CORE_API void ToString(FMemoryToStringContext& OutContext) const;
	LAYOUT_TOSTRING(ToString);
public:
	using ElementType = TCHAR;

	FMemoryImageString() = default;
	FMemoryImageString(FMemoryImageString&&) = default;
	FMemoryImageString(const FMemoryImageString&) = default;
	FMemoryImageString& operator=(FMemoryImageString&&) = default;
	FMemoryImageString& operator=(const FMemoryImageString&) = default;

	UE_FORCEINLINE_HINT FMemoryImageString(const FString& Other) : Data(Other.GetCharArray()) {}

	template <
		typename CharType,
		typename = typename TEnableIf<TIsCharType<CharType>::Value>::Type // This TEnableIf is to ensure we don't instantiate this constructor for non-char types, like id* in Obj-C
	>
		inline FMemoryImageString(const CharType* Src)
	{
		if (Src && *Src)
		{
			int32 SrcLen = TCString<CharType>::Strlen(Src) + 1;
			int32 DestLen = FPlatformString::ConvertedLength<TCHAR>(Src, SrcLen);
			Data.AddUninitialized(DestLen);
			FPlatformString::Convert(Data.GetData(), DestLen, Src, SrcLen);
		}
	}

	UE_FORCEINLINE_HINT operator FString() const
	{
		return FString::ConstructFromPtrSize(Data.GetData(), Len());
	}

	UE_FORCEINLINE_HINT const TCHAR* operator*() const
	{
		return Data.Num() ? Data.GetData() : TEXT("");
	}

	UE_FORCEINLINE_HINT bool IsEmpty() const { return Data.Num() <= 1; }
	UE_FORCEINLINE_HINT SIZE_T GetAllocatedSize() const { return Data.GetAllocatedSize(); }

	UE_FORCEINLINE_HINT int32 Len() const
	{
		return Data.Num() ? Data.Num() - 1 : 0;
	}

	friend inline const TCHAR* GetData(const FMemoryImageString& String)
	{
		return *String;
	}

	friend inline int32 GetNum(const FMemoryImageString& String)
	{
		return String.Len();
	}

	friend inline FArchive& operator<<(FArchive& Ar, FMemoryImageString& Ref)
	{
		Ar << Ref.Data;
		return Ar;
	}

	inline bool operator==(const FMemoryImageString& Rhs) const
	{
		return FCString::Stricmp(**this, *Rhs) == 0;
	}

	inline bool operator!=(const FMemoryImageString& Rhs) const
	{
		return FCString::Stricmp(**this, *Rhs) != 0;
	}

	inline bool operator==(const FString& Rhs) const
	{
		return FCString::Stricmp(**this, *Rhs) == 0;
	}

	inline bool operator!=(const FString& Rhs) const
	{
		return FCString::Stricmp(**this, *Rhs) != 0;
	}

	inline DataType::ElementAllocatorType& GetAllocatorInstance() { return Data.GetAllocatorInstance(); }

	/** Case insensitive string hash function. */
	friend UE_FORCEINLINE_HINT uint32 GetTypeHash(const FMemoryImageString& S)
	{
		return FCrc::Strihash_DEPRECATED(*S);
	}
};

#if WITH_EDITORONLY_DATA
struct FHashedNameDebugString
{
	TMemoryImagePtr<const char> String;
};

namespace Freeze
{
	void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const FHashedNameDebugString& Object, const FTypeLayoutDesc&);
	uint32 IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const FHashedNameDebugString& Object, void* OutDst);
}

DECLARE_INTRINSIC_TYPE_LAYOUT(FHashedNameDebugString);

#endif // WITH_EDITORONLY_DATA

class FHashedName
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FHashedName, CORE_API, NonVirtual);
public:
	inline FHashedName() : Hash(0u) {}
	CORE_API explicit FHashedName(uint64 InHash);
	CORE_API FHashedName(const FHashedName& InName);
	CORE_API FHashedName& operator=(const FHashedName& InName);
	CORE_API FHashedName(const TCHAR* InString);
	CORE_API FHashedName(const FString& InString);
	CORE_API FHashedName(const FName& InName);

	inline uint64 GetHash() const { return Hash; }
	inline bool IsNone() const { return Hash == 0u; }

#if WITH_EDITORONLY_DATA
	const FHashedNameDebugString& GetDebugString() const { return DebugString; }
#endif

	inline bool operator==(const FHashedName& Rhs) const { return Hash == Rhs.Hash; }
	inline bool operator!=(const FHashedName& Rhs) const { return Hash != Rhs.Hash; }

	/** For sorting by name */
	inline bool operator<(const FHashedName& Rhs) const { return Hash < Rhs.Hash; }

	friend inline FArchive& operator<<(FArchive& Ar, FHashedName& String)
	{
		Ar << String.Hash;
		return Ar;
	}

	friend inline uint32 GetTypeHash(const FHashedName& Name)
	{
		return GetTypeHash(Name.Hash);
	}

	/*inline FString ToString() const
	{
		return FString::Printf(TEXT("0x%016X"), Hash);
	}*/

private:
	LAYOUT_FIELD(uint64, Hash);
	LAYOUT_FIELD_EDITORONLY(FHashedNameDebugString, DebugString);

#if WITH_EDITOR
	// Compact binary API with hidden friend operator<<
	CORE_API void Save(FCbWriter& Writer) const;
	bool TryLoad(FCbFieldView Field);
	friend inline FCbWriter& operator<<(FCbWriter& Writer, const FHashedName& Value)
	{
		Value.Save(Writer);
		return Writer;
	}
	friend CORE_API bool LoadFromCompactBinary(FCbFieldView Field, FHashedName& OutValue);
#endif
};

namespace Freeze
{
	CORE_API void IntrinsicToString(const FHashedName& Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
}


class FPtrTableBase
{
public:
	template<typename PtrType>
	static void LoadAndApplyPatchesFromArchive(FArchive& Ar, void* FrozenBase, const PtrType& Ptr)
	{
		int32 NumOffsets = 0;
		Ar << NumOffsets;
		for (int32 OffsetIndex = 0; OffsetIndex < NumOffsets; ++OffsetIndex)
		{
			uint32 Offset = 0u;
			Ar << Offset;
			new((char*)FrozenBase + Offset) PtrType(Ptr);
		}
	}

	CORE_API void SavePatchesToArchive(FArchive& Ar, uint32 PtrIndex) const;

protected:
	struct FPatchOffset
	{
		uint32 Offset;
		uint32 NextIndex;
	};

	struct FPatchOffsetList
	{
		FPatchOffsetList() : FirstIndex(~0u), NumOffsets(0u) {}
		uint32 FirstIndex;
		uint32 NumOffsets;
	};

	CORE_API void AddPatchedPointerBase(uint32 PtrIndex, uint64 Offset);

	TArray<FPatchOffsetList> PatchLists;
	TArray<FPatchOffset> PatchOffsets;
};

template<typename T, typename PtrType>
class TPtrTableBase : public FPtrTableBase
{
public:
	static const FTypeLayoutDesc& StaticGetPtrTypeLayoutDesc();

	void Empty(int32 NewSize = 0)
	{
		Pointers.Reset(NewSize);
	}

	uint32 Num() const { return Pointers.Num(); }
	uint32 AddIndexedPointer(T* Ptr) { check(Ptr); return Pointers.AddUnique(Ptr); }

	bool TryAddIndexedPtr(const FTypeLayoutDesc& TypeDesc, void* Ptr, int32& OutIndex)
	{
		if (TypeDesc == StaticGetPtrTypeLayoutDesc())
		{
			OutIndex = AddIndexedPointer(static_cast<T*>(Ptr));
			return true;
		}
		return false;
	}

	void LoadIndexedPointer(T* Ptr)
	{
		if (Ptr)
		{
			checkSlow(!Pointers.Contains(Ptr));
			Pointers.Add(Ptr);
		}
		else
		{
			// allow duplicate nullptrs
			// pointers that were valid when saving may not be found when loading, need to preserve indices
			Pointers.Add(nullptr);
		}
	}

	void AddPatchedPointer(T* Ptr, uint64 Offset)
	{
		const uint32 PtrIndex = AddIndexedPointer(Ptr);
		FPtrTableBase::AddPatchedPointerBase(PtrIndex, Offset);
	}

	T* GetIndexedPointer(uint32 i) const { return Pointers[i]; }

	bool TryGetIndexedPtr(const FTypeLayoutDesc& TypeDesc, uint32 i, void*& OutPtr) const
	{
		if (TypeDesc == StaticGetPtrTypeLayoutDesc())
		{
			OutPtr = GetIndexedPointer(i);
			return true;
		}
		return false;
	}

	void ApplyPointerPatches(void* FrozenBase) const
	{
		for (int32 PtrIndex = 0; PtrIndex < PatchLists.Num(); ++PtrIndex)
		{
			uint32 PatchIndex = PatchLists[PtrIndex].FirstIndex;
			while (PatchIndex != ~0u)
			{
				const FPatchOffset& Patch = PatchOffsets[PatchIndex];
				new((char*)FrozenBase + Patch.Offset) PtrType(Pointers[PtrIndex]);
				PatchIndex = Patch.NextIndex;
			}
		}
	}

	inline typename TArray<PtrType>::RangedForIteratorType begin() { return Pointers.begin(); }
	inline typename TArray<PtrType>::RangedForIteratorType end() { return Pointers.end(); }
	inline typename TArray<PtrType>::RangedForConstIteratorType begin() const { return Pointers.begin(); }
	inline typename TArray<PtrType>::RangedForConstIteratorType end() const { return Pointers.end(); }
private:
	TArray<PtrType> Pointers;
};

template<typename T>
class TPtrTable : public TPtrTableBase<T, T*> {};

template<typename T>
class TRefCountPtrTable : public TPtrTableBase<T, TRefCountPtr<T>>
{
	using Super = TPtrTableBase<T, TRefCountPtr<T>>;
};

class FVoidPtrTable : public TPtrTableBase<void, void*> {};

#ifdef _MSC_VER
#pragma warning(push)
#pragma warning(disable : 4583) // destructor is not implicitly called
#endif

/**
 * Note that IndexedPtr types don't declare a default intrinsic type layout.
 * Instead any required pointer types need to be declared/implemented using DECLARE_EXPORTED_TEMPLATE_INTRINSIC_TYPE_LAYOUT/IMPLEMENT_TEMPLATE_INTRINSIC_TYPE_LAYOUT.
 * The TypeDesc of indexed pointers are compared for equality when adding to pointer tables,
 * and it's possible for inline type layouts to generate multiple copies when referenced from multiple modules
 */
template<typename T, typename PtrType>
class TIndexedPtrBase
{
public:
	using FPtrTable = TPtrTableBase<T, PtrType>;

	inline TIndexedPtrBase(T* InPtr = nullptr) : Ptr(InPtr) {}
	inline ~TIndexedPtrBase() { if(!IsFrozen()) Ptr.~PtrType(); }

	// Copy constructor requires an unfrozen source
	inline TIndexedPtrBase(const TIndexedPtrBase<T, PtrType>& Rhs) : Ptr(Rhs.GetUnfrozen()) {}

	inline TIndexedPtrBase(const TIndexedPtrBase<T, PtrType>& Rhs, const FPtrTable& InTable) : Ptr(Rhs.Get(InTable)) {}

	inline TIndexedPtrBase& operator=(T* Rhs)
	{
		// If not currently frozen, invoke the standard assignment operator for the underlying pointer type
		// If frozen, construct a new (non-frozen) pointer over the existing frozen offset
		if (!IsFrozen()) Ptr = Rhs;
		else new(&Ptr) PtrType(Rhs);
		check(!IsFrozen());
		return *this;
	}

	inline TIndexedPtrBase& operator=(const PtrType& Rhs)
	{
		if (!IsFrozen()) Ptr = Rhs;
		else new(&Ptr) PtrType(Rhs);
		check(!IsFrozen());
		return *this;
	}

	inline TIndexedPtrBase& operator=(PtrType&& Rhs)
	{
		if (!IsFrozen()) Ptr = Rhs;
		else new(&Ptr) PtrType(Rhs);
		check(!IsFrozen());
		return *this;
	}
	
	inline bool IsFrozen() const { return PackedIndex & IsFrozenMask; }
	inline bool IsValid() const { return PackedIndex != 0u; } // works for both frozen/unfrozen cases
	inline bool IsNull() const { return PackedIndex == 0u; }

	inline void SafeRelease()
	{
		if (!IsFrozen())
		{
			SafeReleaseImpl(Ptr);
		}
	}

	inline T* Get(const FPtrTable& PtrTable) const
	{
		if (IsFrozen())
		{
			return PtrTable.GetIndexedPointer((uint32)(PackedIndex >> IndexShift));
		}
		return Ptr;
	}

	inline T* Get(const FPointerTableBase* PtrTable) const
	{
		if (IsFrozen())
		{
			check(PtrTable);
			const FTypeLayoutDesc& TypeDesc = StaticGetTypeLayoutDesc<TIndexedPtrBase<T, PtrType>>();
			return static_cast<T*>(PtrTable->GetIndexedPointer(TypeDesc, (uint32)(PackedIndex >> IndexShift)));
		}
		return Ptr;
	}

	inline T* GetUnfrozen() const { check(!IsFrozen()); return Ptr; }

private:
	enum
	{
		IsFrozenMask = (1 << 0),
		IndexShift = 1,
	};

	static void SafeReleaseImpl(T*& InPtr)
	{
		if (InPtr)
		{
			delete InPtr;
			InPtr = nullptr;
		}
	}

	static void SafeReleaseImpl(TRefCountPtr<T>& InPtr)
	{
		InPtr.SafeRelease();
	}

	static_assert(sizeof(PtrType) <= sizeof(uint64), "PtrType must fit within a standard pointer");
	union
	{
		PtrType Ptr;
		uint64 PackedIndex;
	};
};

#ifdef _MSC_VER
#pragma warning(pop)
#endif

template<typename T, typename PtrType>
inline const FTypeLayoutDesc& TPtrTableBase<T, PtrType>::StaticGetPtrTypeLayoutDesc()
{
	return StaticGetTypeLayoutDesc<TIndexedPtrBase<T, PtrType>>();
}

namespace Freeze
{
	template<typename T, typename PtrType>
	void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const TIndexedPtrBase<T, PtrType>& Object, const FTypeLayoutDesc& TypeDesc)
	{
		T* RawPtr = Object.Get(Writer.TryGetPrevPointerTable());
		if (RawPtr)
		{
			const uint32 Index = Writer.GetPointerTable().AddIndexedPointer(TypeDesc, RawPtr);
			check(Index != (uint32)INDEX_NONE);
			const uint64 FrozenPackedIndex = ((uint64)Index << 1u) | 1u;
			Writer.WriteBytes(FrozenPackedIndex);
		}
		else
		{
			Writer.WriteBytes(uint64(0u));
		}
	}

	template<typename T, typename PtrType>
	uint32 IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const TIndexedPtrBase<T, PtrType>& Object, void* OutDst)
	{
		new(OutDst) TIndexedPtrBase<T, PtrType>(Object.Get(Context.TryGetPrevPointerTable()));
		return sizeof(Object);
	}

	template<typename T, typename PtrType>
	uint32 IntrinsicAppendHash(const TIndexedPtrBase<T, PtrType>* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		return AppendHashForNameAndSize(TypeDesc.Name, sizeof(TIndexedPtrBase<T, PtrType>), Hasher);
	}

	template<typename T, typename PtrType>
	inline uint32 IntrinsicGetTargetAlignment(const TIndexedPtrBase<T, PtrType>* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams)
	{
		return FMath::Min(8u, LayoutParams.MaxFieldAlignment);
	}
}

template<typename T>
using TIndexedPtr = TIndexedPtrBase<T, T*>;

template<typename T>
using TIndexedRefCountPtr = TIndexedPtrBase<T, TRefCountPtr<T>>;

template<typename T, typename PtrType>
class TPatchedPtrBase
{
public:
	using FPtrTable = TPtrTableBase<T, PtrType>;

	inline TPatchedPtrBase(T* InPtr = nullptr) : Ptr(InPtr) {}

	inline T* Get() const
	{
		return Ptr;
	}

	inline T* GetChecked() const { T* Value = Get(); check(Value); return Value; }
	inline T* operator->() const { return GetChecked(); }
	inline T& operator*() const { return *GetChecked(); }
	inline operator T*() const { return Get(); }

private:
	static_assert(sizeof(PtrType) == sizeof(void*), "PtrType must be a standard pointer");
	PtrType Ptr;
};

template<typename T>
using TPatchedPtr = TPatchedPtrBase<T, T*>;

template<typename T>
using TPatchedRefCountPtr = TPatchedPtrBase<T, TRefCountPtr<T>>;

