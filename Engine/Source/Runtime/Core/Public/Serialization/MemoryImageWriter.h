// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Serialization/MemoryLayout.h"

struct FScriptName;
class FMemoryImage;
class FMemoryImageSection;
class FPointerTableBase;

class FMemoryImageWriter
{
public:
	CORE_API explicit FMemoryImageWriter(FMemoryImage& InImage);
	CORE_API explicit FMemoryImageWriter(FMemoryImageSection* InSection);
	CORE_API ~FMemoryImageWriter();

	CORE_API FMemoryImage& GetImage() const;
	CORE_API const FPlatformTypeLayoutParameters& GetHostLayoutParams() const;
	CORE_API const FPlatformTypeLayoutParameters& GetTargetLayoutParams() const;
	CORE_API FPointerTableBase& GetPointerTable() const;
	CORE_API const FPointerTableBase* TryGetPrevPointerTable() const;

	inline bool Is32BitTarget() const { return GetTargetLayoutParams().Is32Bit(); }
	inline bool Is64BitTarget() const { return !Is32BitTarget(); }

	CORE_API int32 AddTypeDependency(const FTypeLayoutDesc& TypeDesc);

	CORE_API void WriteObject(const void* Object, const FTypeLayoutDesc& TypeDesc);
	CORE_API void WriteObjectArray(const void* Object, const FTypeLayoutDesc& TypeDesc, uint32_t NumArray);
	CORE_API void WriteRootObject(const void* Object, const FTypeLayoutDesc& TypeDesc);

	CORE_API uint32 GetOffset() const;
	CORE_API uint32 WriteAlignment(uint32 Alignment);
	CORE_API void WritePaddingToSize(uint32 Offset);
	CORE_API uint32 WriteBytes(const void* Data, uint32 Size);
	CORE_API FMemoryImageWriter WritePointer(const FTypeLayoutDesc& StaticTypeDesc, const FTypeLayoutDesc& DerivedTypeDesc, uint32* OutOffsetToBase = nullptr);
	CORE_API FMemoryImageWriter WritePointer(const FTypeLayoutDesc& TypeDesc);
	CORE_API uint32 WriteNullPointer();
	CORE_API uint32 WriteRawPointerSizedBytes(uint64 PointerValue);
	CORE_API uint32 WriteVTable(const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& DerivedTypeDesc);
	CORE_API uint32 WriteFMemoryImageName(int32 NumBytes, const FName& Name);
	CORE_API uint32 WriteFScriptName(const FScriptName& Name);

	template<typename T>
	void WriteObject(const T& Object)
	{
		const FTypeLayoutDesc& TypeDesc = GetTypeLayoutDesc(TryGetPrevPointerTable(), Object);
		WriteObject(&Object, TypeDesc);
	}

	template<typename T>
	void WriteRootObject(const T& Object)
	{
		const FTypeLayoutDesc& TypeDesc = GetTypeLayoutDesc(TryGetPrevPointerTable(), Object);
		WriteRootObject(&Object, TypeDesc);
	}

	template<typename T>
	uint32 WriteAlignment()
	{
		return WriteAlignment(alignof(T));
	}

	template<typename T>
	uint32 WriteBytes(const T& Data)
	{
		return WriteBytes(&Data, sizeof(T));
	}

//private:
	FMemoryImageSection* Section;
};

class FMemoryUnfreezeContent
{
public:
	explicit FMemoryUnfreezeContent(const FPointerTableBase* InPointerTable)
		: PrevPointerTable(InPointerTable)
		, bIsFrozenForCurrentPlatform(true)
	{
		FrozenLayoutParameters.InitializeForCurrent();
	}

	FMemoryUnfreezeContent(const FPointerTableBase* InPointerTable, const FPlatformTypeLayoutParameters& InLayoutParams)
		: PrevPointerTable(InPointerTable)
		, FrozenLayoutParameters(InLayoutParams)
		, bIsFrozenForCurrentPlatform(InLayoutParams.IsCurrentPlatform())
	{
	}

	inline const FPointerTableBase* TryGetPrevPointerTable() const { return PrevPointerTable; }

	CORE_API const FTypeLayoutDesc* GetDerivedTypeDesc(const FTypeLayoutDesc& StaticTypeDesc, int32 TypeIndex) const;

	inline uint32 UnfreezeObject(const void* Object, const FTypeLayoutDesc& TypeDesc, void* OutDst) const
	{
		return TypeDesc.UnfrozenCopyFunc(*this, Object, TypeDesc, OutDst);
	}

	template<typename T>
	inline uint32 UnfreezeObject(const T& Object, void* OutDst) const
	{
		const FTypeLayoutDesc& TypeDesc = GetTypeLayoutDesc(TryGetPrevPointerTable(), Object);
		return UnfreezeObject(&Object, TypeDesc, OutDst);
	}

	const FPointerTableBase* PrevPointerTable = nullptr;

	// Layout of the frozen data
	FPlatformTypeLayoutParameters FrozenLayoutParameters;
	bool bIsFrozenForCurrentPlatform; // FrozenLayoutParameters.IsCurrentPlatform
};
