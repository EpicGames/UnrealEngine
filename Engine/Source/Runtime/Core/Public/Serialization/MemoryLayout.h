// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/StaticClassProvider.h"
#include "Concepts/StaticStructProvider.h"
#include "Containers/EnumAsByte.h"
#include "Containers/StringFwd.h"
#include "HAL/UnrealMemory.h"
#include "Misc/DelayedAutoRegister.h"
#include "Templates/EnableIf.h"
#include "Templates/IsAbstract.h"
#include "Templates/IsPolymorphic.h"
#include "Templates/Models.h"
#include "Templates/UnrealTemplate.h"

class FCbFieldView;
class FCbWriter;
class FHashedName;
class FShaderKeyGenerator;
class FSHA1;
class FMemoryImageWriter;
class FMemoryUnfreezeContent;
class FPointerTableBase;
class ITargetPlatform;
struct FTypeLayoutDesc;
struct FPlatformTypeLayoutParameters;

/*#define UE_STATIC_ONLY(T) \
	T() = delete; \
	~T() = delete; \
	UE_NONCOPYABLE(T)*/
#define UE_STATIC_ONLY(T)

/**
 * Certain compilers don't accept full explicit template specializations inside class declarations.
 * Partial specializations are accepted however.  So for these platforms, we force partial specialization via an extra dummy parameter that's otherwise ignored.
 * This actually breaks MSVC, as MSVC doesn't allow complex statements as template values during partial specialization, so need use full specialization there.
 */
#if PLATFORM_ANDROID || PLATFORM_MAC || PLATFORM_IOS || PLATFORM_TVOS
#define UE_DECLARE_INTERNAL_LINK_BASE(T) template<int Counter, typename Dummy=void> struct T
#define UE_DECLARE_INTERNAL_LINK_SPECIALIZATION(T, Counter) template<typename Dummy> struct T<Counter, Dummy>
#else
#define UE_DECLARE_INTERNAL_LINK_BASE(T) template<int Counter> struct T
#define UE_DECLARE_INTERNAL_LINK_SPECIALIZATION(T, Counter) template<> struct T<Counter>
#endif

template<typename T> const FTypeLayoutDesc& StaticGetTypeLayoutDesc();

struct FMemoryToStringContext
{
	CORE_API void AppendIndent();
	CORE_API void AppendNullptr();
	CORE_API void AppendUnfrozenPointer(const FTypeLayoutDesc& StaticTypeDesc);
	CORE_API void AppendFrozenPointer(const FTypeLayoutDesc& StaticTypeDesc, int32 FrozenTypeIndex);

	const FPointerTableBase* TryGetPrevPointerTable() const { return PrevPointerTable; }

	FStringBuilderBase* String = nullptr;
	const FPointerTableBase* PrevPointerTable = nullptr;
	int32 Indent = 0;
};

namespace ETypeLayoutInterface
{
	enum Type : uint8
	{
		NonVirtual,
		Virtual,
		Abstract,
	};

	inline bool HasVTable(Type InType) { return InType != NonVirtual; }
}

namespace EFieldLayoutFlags
{
	enum Type : uint8
	{
		None = 0u,
		WithEditorOnly = (1u << 0),
		WithRayTracing = (1u << 1),
		Transient = (1u << 2),
		UseInstanceWithNoProperty = (1u << 3),
	};

	UE_FORCEINLINE_HINT Type MakeFlags(uint32 Flags = None) { return (Type)Flags; }
	UE_FORCEINLINE_HINT Type MakeFlagsEditorOnly(uint32 Flags = None) { return (Type)(WithEditorOnly | Flags); }
	UE_FORCEINLINE_HINT Type MakeFlagsRayTracing(uint32 Flags = None) { return (Type)(WithRayTracing | Flags); }
}

struct FFieldLayoutDesc
{
	typedef void (FWriteFrozenMemoryImageFunc)(FMemoryImageWriter& Writer, const void* Object, const void* FieldObject, const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& DerivedTypeDesc);

	const TCHAR* Name;
	const struct FTypeLayoutDesc* Type;
	const FFieldLayoutDesc* Next;
	FWriteFrozenMemoryImageFunc* WriteFrozenMemoryImageFunc;
	uint32 Offset;
	uint32 NumArray;
	EFieldLayoutFlags::Type Flags;
	uint8 BitFieldSize;
	uint8 UFieldNameLength; // this is the number of characters in Name, omitting any _DEPRECATED suffix
};

struct FTypeLayoutDesc
{
	typedef void (FDestroyFunc)(void* Object, const FTypeLayoutDesc& TypeDesc, const FPointerTableBase* PtrTable, bool bIsFrozen);
	typedef void (FWriteFrozenMemoryImageFunc)(FMemoryImageWriter& Writer, const void* Object, const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& DerivedTypeDesc);
	typedef uint32 (FUnfrozenCopyFunc)(const FMemoryUnfreezeContent& Context, const void* Object, const FTypeLayoutDesc& TypeDesc, void* OutDst);
	typedef uint32 (FAppendHashFunc)(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher);
	typedef uint32 (FGetTargetAlignmentFunc)(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams);
	typedef void (FToStringFunc)(const void* Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	typedef const void* (FGetDefaultFunc)();

	static CORE_API const FTypeLayoutDesc& GetInvalidTypeLayout();

	static CORE_API void Initialize(FTypeLayoutDesc& TypeDesc);
	static CORE_API void Register(FTypeLayoutDesc& TypeDesc);
	static CORE_API const FTypeLayoutDesc* Find(uint64 NameHash);

	CORE_API uint32 GetOffsetToBase(const FTypeLayoutDesc& BaseTypeDesc) const;
	CORE_API bool IsDerivedFrom(const FTypeLayoutDesc& BaseTypeDesc) const;

	const FTypeLayoutDesc* HashNext;
	const TCHAR* Name;
	const FFieldLayoutDesc* Fields;
	FDestroyFunc* DestroyFunc;
	FWriteFrozenMemoryImageFunc* WriteFrozenMemoryImageFunc;
	FUnfrozenCopyFunc* UnfrozenCopyFunc;
	FAppendHashFunc* AppendHashFunc;
	FGetTargetAlignmentFunc* GetTargetAlignmentFunc;
	FToStringFunc* ToStringFunc;
	FGetDefaultFunc* GetDefaultObjectFunc;

	uint64 NameHash; // from FHashedName(Name)
	uint32 Size;
	uint32 SizeFromFields;
	uint32 Alignment;
	ETypeLayoutInterface::Type Interface;
	uint8 NumBases;
	uint8 NumVirtualBases;
	uint8 IsIntrinsic : 1;
	UE_DEPRECATED(5.8, "IsInitialized is not used anymore")
	uint8 IsInitialized : 1;

	friend inline bool operator==(const FTypeLayoutDesc& Lhs, const FTypeLayoutDesc& Rhs) { return &Lhs == &Rhs; }
	friend inline bool operator!=(const FTypeLayoutDesc& Lhs, const FTypeLayoutDesc& Rhs) { return &Lhs != &Rhs; }
};

struct FRegisterTypeLayoutDesc
{
	explicit CORE_API FRegisterTypeLayoutDesc(FTypeLayoutDesc& TypeDesc);
	CORE_API FRegisterTypeLayoutDesc(const TCHAR* Name, FTypeLayoutDesc& TypeDesc);
};

template<typename T, typename Base>
inline uint32 GetBaseOffset()
{
	alignas(T) char Dummy[sizeof(T)];
	T* Derived = reinterpret_cast<T*>(Dummy);
	return (uint32)((uint64)static_cast<Base*>(Derived) - (uint64)Derived);
}

/**
 * Access to a global default object is required in order to patch vtables
 * Normally this can be provided by a default-constructed object.  For objects without default constructors, a default object must be provided through the global function GetDefault<T>()
 * THasCustomDefaultObject<T>::Value must be specialized to true for these types.  This is done by default for any UObject-derived types, which already provide GetDefault<T>()
 */
struct CProvidesDefaultUObject
{
	template<typename T>
	auto Requires(const T&) -> decltype(T::StaticClass()->GetDefaultObject());
};

template<typename T>
const T* GetDefault();

template<typename T>
struct THasCustomDefaultObject
{
	static constexpr bool Value = TModels_V<CProvidesDefaultUObject, T>;
};

template<typename T>
const T* InternalGetDefaultObject()
{
	if constexpr (THasCustomDefaultObject<T>::Value)
	{
		return GetDefault<T>();
	}
	else
	{
		static const T Default;
		return &Default;
	}
}

template<typename T, ETypeLayoutInterface::Type InterfaceType>
struct TGetDefaultObjectHelper
{
	UE_STATIC_ONLY(TGetDefaultObjectHelper);
	static const void* Do() { return nullptr; }
};

template<typename T>
struct TGetDefaultObjectHelper<T, ETypeLayoutInterface::Virtual>
{
	UE_STATIC_ONLY(TGetDefaultObjectHelper);
	static const void* Do() { return reinterpret_cast<const void*>(InternalGetDefaultObject<T>()); }
};


template<typename T, ETypeLayoutInterface::Type InterfaceType>
struct TValidateInterfaceHelper;

template<typename T> struct TValidateInterfaceHelper<T, ETypeLayoutInterface::NonVirtual>
{
	UE_STATIC_ONLY(TValidateInterfaceHelper);
	static constexpr bool Value = !TIsPolymorphic<T>::Value;
};

template<typename T> struct TValidateInterfaceHelper<T, ETypeLayoutInterface::Virtual>
{
	UE_STATIC_ONLY(TValidateInterfaceHelper);
	static constexpr bool Value = !TIsAbstract<T>::Value;
};

template<typename T> struct TValidateInterfaceHelper<T, ETypeLayoutInterface::Abstract>
{
	UE_STATIC_ONLY(TValidateInterfaceHelper);
	static constexpr bool Value = true;
};

namespace Freeze
{
	CORE_API void DefaultWriteMemoryImageField(FMemoryImageWriter& Writer, const void* Object, const void* FieldObject, const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& DerivedTypeDesc);
	CORE_API void DefaultWriteMemoryImage(FMemoryImageWriter& Writer, const void* Object, const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& DerivedTypeDesc);
	CORE_API uint32 DefaultUnfrozenCopy(const FMemoryUnfreezeContent& Context, const void* Object, const FTypeLayoutDesc& TypeDesc, void* OutDst);
	CORE_API uint32 DefaultAppendHash(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher);
	CORE_API uint32 DefaultGetTargetAlignment(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams);
	CORE_API void DefaultToString(const void* Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API uint32 AppendHashForNameAndSize(const TCHAR* Name, uint32 Size, FSHA1& Hasher);
	CORE_API void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const void* Object, uint32 Size);
	CORE_API void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, void*, const FTypeLayoutDesc&);

	// Override for types that need access to a PointerTable in order to destroy frozen data
	// WARNING! Frozen data may have deduplicated portions, so it's possible that a given object may have multiple references
	// Any attempt to cleanup/destroy frozen data must take care to safely allow DestroyObject to be called multiple times on the same frozen object
	// Generally speaking, it's *NOT* safe to invoke the built-in destructor multiple times
	template<typename T>
	inline void DestroyObject(T* Object, const FPointerTableBase* PtrTable, bool bIsFrozen)
	{
		// Only call destructor for non-frozen objects
		if (!bIsFrozen)
		{
			Object->~T();
		}
		// Wipe destroyed memory to a common pattern
		FMemory::Memset(Object, 0xfe, sizeof(T));
	}

	template<typename T>
	UE_FORCEINLINE_HINT void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const T& Object, const FTypeLayoutDesc& TypeDesc)
	{
		IntrinsicWriteMemoryImage(Writer, &Object, sizeof(T));
	}

	template<typename T>
	inline uint32 IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const T& Object, void* OutDst)
	{
		new(OutDst) T(Object);
		return sizeof(T);
	}

	template<typename T>
	UE_FORCEINLINE_HINT uint32 IntrinsicAppendHash(const T* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		return DefaultAppendHash(TypeDesc, LayoutParams, Hasher);
	}

	template<typename T>
	UE_FORCEINLINE_HINT uint32 IntrinsicGetTargetAlignment(const T* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams)
	{
		return TypeDesc.Alignment;
	}

	template<typename T>
	inline void IntrinsicToString(const T& Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
	{
		DefaultToString(&Object, TypeDesc, LayoutParams, OutContext);
	}

	CORE_API uint32 IntrinsicAppendHash(void* const* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher);
	CORE_API uint32 IntrinsicGetTargetAlignment(void* const* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams);

	CORE_API void IntrinsicToString(char Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API void IntrinsicToString(short Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API void IntrinsicToString(int Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API void IntrinsicToString(int8 Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API void IntrinsicToString(long Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API void IntrinsicToString(long long Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API void IntrinsicToString(unsigned char Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API void IntrinsicToString(unsigned short Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API void IntrinsicToString(unsigned int Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API void IntrinsicToString(unsigned long Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API void IntrinsicToString(unsigned long long Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API void IntrinsicToString(float Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API void IntrinsicToString(double Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API void IntrinsicToString(wchar_t Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API void IntrinsicToString(char16_t Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	CORE_API void IntrinsicToString(void* Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);

	// helper functions
	CORE_API void ExtractBitFieldValue(const void* Value, uint32 SrcBitOffset, uint32 DestBitOffset, uint32 NumBits, uint64& InOutValue);
	CORE_API bool IncludeField(const FFieldLayoutDesc* FieldDesc, const FPlatformTypeLayoutParameters& LayoutParams);
	CORE_API uint32 GetTargetAlignment(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams);

	CORE_API uint32 AppendHash(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher);
	CORE_API uint32 AppendHashPair(const FTypeLayoutDesc& KeyTypeDesc, const FTypeLayoutDesc& ValueTypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher);
	CORE_API uint32 HashLayout(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHAHash& OutHash);
	CORE_API uint32 HashLayouts(const TArray<const FTypeLayoutDesc*>& TypeLayouts, const FPlatformTypeLayoutParameters& LayoutParams, FSHAHash& OutHash);
	CORE_API FSHAHash HashLayout(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams);
}

template<typename T>
struct TUsePropertyFreezing
{
	static constexpr bool Value = (TModels_V<CStaticClassProvider, T> || TModels_V<CStaticStructProvider, T>);
};

template <typename T, bool bUsePropertyFreezing=TUsePropertyFreezing<T>::Value>
struct TGetFreezeImageHelper
{
	static UE_FORCEINLINE_HINT FTypeLayoutDesc::FWriteFrozenMemoryImageFunc* Do() { return &Freeze::DefaultWriteMemoryImage; }
};

template <typename T, bool bProvidesStaticStruct=TModels_V<CStaticStructProvider, T>>
struct TGetFreezeImageFieldHelper
{
	static UE_FORCEINLINE_HINT FFieldLayoutDesc::FWriteFrozenMemoryImageFunc* Do() { return &Freeze::DefaultWriteMemoryImageField; }
};

template<typename T, typename BaseType>
static void InternalInitializeBaseHelper(FTypeLayoutDesc& TypeDesc)
{
	alignas(FFieldLayoutDesc) static uint8 FieldBuffer[sizeof(FFieldLayoutDesc)] = { 0 };
	FFieldLayoutDesc& FieldDesc = *(FFieldLayoutDesc*)FieldBuffer;
	FieldDesc.Name = TEXT("BASE");
	FieldDesc.UFieldNameLength = 4;
	FieldDesc.Type = &StaticGetTypeLayoutDesc<BaseType>();
	FieldDesc.WriteFrozenMemoryImageFunc = TGetFreezeImageFieldHelper<UE_REMOVE_OPTIONAL_PARENS(T)>::Do();
	FieldDesc.Offset = GetBaseOffset<T, BaseType>();
	FieldDesc.NumArray = 1u;
	FieldDesc.Next = TypeDesc.Fields;
	TypeDesc.Fields = &FieldDesc;
	++TypeDesc.NumBases;
	if (ETypeLayoutInterface::HasVTable(FieldDesc.Type->Interface)) ++TypeDesc.NumVirtualBases;
}

template<typename T>
static UE_FORCEINLINE_HINT void InternalInitializeBasesHelper(FTypeLayoutDesc& TypeDesc) {}

template<typename T, typename BaseType, typename ...RemainingBaseTypes>
static void InternalInitializeBasesHelper(FTypeLayoutDesc& TypeDesc)
{
	InternalInitializeBasesHelper<T, RemainingBaseTypes...>(TypeDesc);
	InternalInitializeBaseHelper<T, BaseType>(TypeDesc);
}

template<typename T>
struct TGetBaseTypeHelper
{
	UE_STATIC_ONLY(TGetBaseTypeHelper);
	template<typename InternalType> static typename InternalType::DerivedType Test(const typename InternalType::DerivedType*);
	template<typename InternalType> static void Test(...);

	using Type = decltype(Test<T>(nullptr));
};

template<typename T, typename BaseType>
struct TInitializeBaseHelper
{
	UE_STATIC_ONLY(TInitializeBaseHelper);
	static void Initialize(FTypeLayoutDesc& TypeDesc) { InternalInitializeBaseHelper<T, BaseType>(TypeDesc); }
};

template<typename T>
struct TInitializeBaseHelper<T, void>
{
	UE_STATIC_ONLY(TInitializeBaseHelper);
	static UE_FORCEINLINE_HINT void Initialize(FTypeLayoutDesc& TypeDesc) {}
};

namespace Freeze
{
	// Finds the length of the field name, omitting any _DEPRECATED suffix
	CORE_API uint8 FindFieldNameLength(const TCHAR* Name);
}

#define INTERNAL_LAYOUT_FIELD(T, InName, InOffset, InFlags, InNumArray, InBitFieldSize, Counter) \
	PRAGMA_DISABLE_DEPRECATION_WARNINGS \
	UE_DECLARE_INTERNAL_LINK_SPECIALIZATION(InternalLinkType, Counter - CounterBase) { \
		UE_STATIC_ONLY(InternalLinkType); \
		static void Initialize(FTypeLayoutDesc& TypeDesc) { \
			InternalLinkType<Counter - CounterBase + 1>::Initialize(TypeDesc); \
			alignas(FFieldLayoutDesc) static uint8 FieldBuffer[sizeof(FFieldLayoutDesc)] = { 0 }; \
			FFieldLayoutDesc& FieldDesc = *(FFieldLayoutDesc*)FieldBuffer; \
			FieldDesc.Name = TEXT(#InName); \
			FieldDesc.UFieldNameLength = Freeze::FindFieldNameLength(FieldDesc.Name); \
			FieldDesc.Type = &StaticGetTypeLayoutDesc<UE_REMOVE_OPTIONAL_PARENS(T)>(); \
			FieldDesc.WriteFrozenMemoryImageFunc = TGetFreezeImageFieldHelper<UE_REMOVE_OPTIONAL_PARENS(T)>::Do(); \
			FieldDesc.Offset = InOffset; \
			FieldDesc.NumArray = InNumArray; \
			FieldDesc.Flags = InFlags; \
			FieldDesc.BitFieldSize = InBitFieldSize; \
			FieldDesc.Next = TypeDesc.Fields; \
			TypeDesc.Fields = &FieldDesc; \
		} \
	}; \
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

#define INTERNAL_LAYOUT_FIELD_WITH_WRITER(T, InName, InFunc, Counter) \
	PRAGMA_DISABLE_DEPRECATION_WARNINGS \
	UE_DECLARE_INTERNAL_LINK_SPECIALIZATION(InternalLinkType, Counter - CounterBase) { \
		UE_STATIC_ONLY(InternalLinkType); \
		static void CallWriteFrozenField(FMemoryImageWriter& Writer, const void* Object, const void* FieldObject, const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& DerivedTypeDesc) { \
			static_cast<const DerivedType*>(Object)->InFunc(Writer, *(UE_REMOVE_OPTIONAL_PARENS(T)*)FieldObject); \
		} \
		static void Initialize(FTypeLayoutDesc& TypeDesc) { \
			InternalLinkType<Counter - CounterBase + 1>::Initialize(TypeDesc); \
			alignas(FFieldLayoutDesc) static uint8 FieldBuffer[sizeof(FFieldLayoutDesc)] = { 0 }; \
			FFieldLayoutDesc& FieldDesc = *(FFieldLayoutDesc*)FieldBuffer; \
			FieldDesc.Name = TEXT(#InName); \
			FieldDesc.UFieldNameLength = Freeze::FindFieldNameLength(FieldDesc.Name); \
			FieldDesc.Type = &StaticGetTypeLayoutDesc<UE_REMOVE_OPTIONAL_PARENS(T)>(); \
			FieldDesc.WriteFrozenMemoryImageFunc = &CallWriteFrozenField; \
			FieldDesc.Offset = STRUCT_OFFSET(DerivedType, InName); \
			FieldDesc.NumArray = 1u; \
			FieldDesc.Flags = EFieldLayoutFlags::None; \
			FieldDesc.Next = TypeDesc.Fields; \
			TypeDesc.Fields = &FieldDesc; \
		} \
	}; \
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#define INTERNAL_LAYOUT_WRITE_MEMORY_IMAGE(Func, Counter) \
	UE_DECLARE_INTERNAL_LINK_SPECIALIZATION(InternalLinkType, Counter - CounterBase) { \
		UE_STATIC_ONLY(InternalLinkType); \
		static void CallWriteMemoryImage(FMemoryImageWriter& Writer, const void* Object, const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& DerivedTypeDesc) { \
			static_cast<const DerivedType*>(Object)->Func(Writer, TypeDesc); \
		} \
		static void Initialize(FTypeLayoutDesc& TypeDesc) { \
			InternalLinkType<Counter - CounterBase + 1>::Initialize(TypeDesc); \
			TypeDesc.WriteFrozenMemoryImageFunc = &CallWriteMemoryImage; \
		} \
	};

#define INTERNAL_LAYOUT_TOSTRING(Func, Counter) \
	UE_DECLARE_INTERNAL_LINK_SPECIALIZATION(InternalLinkType, Counter - CounterBase) { \
		UE_STATIC_ONLY(InternalLinkType); \
		static void CallToString(const void* Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext) { \
			static_cast<const DerivedType*>(Object)->Func(OutContext); \
		} \
		static void Initialize(FTypeLayoutDesc& TypeDesc) { \
			InternalLinkType<Counter - CounterBase + 1>::Initialize(TypeDesc); \
			TypeDesc.ToStringFunc = &CallToString; \
		} \
	};

#define LAYOUT_FIELD(T, Name, ...) UE_REMOVE_OPTIONAL_PARENS(T) Name; INTERNAL_LAYOUT_FIELD(T, Name, STRUCT_OFFSET(DerivedType, Name), EFieldLayoutFlags::MakeFlags(__VA_ARGS__), 1u, 0u, __COUNTER__)
#define LAYOUT_MUTABLE_FIELD(T, Name, ...) mutable UE_REMOVE_OPTIONAL_PARENS(T) Name; INTERNAL_LAYOUT_FIELD(T, Name, STRUCT_OFFSET(DerivedType, Name), EFieldLayoutFlags::MakeFlags(__VA_ARGS__), 1u, 0u, __COUNTER__)
#define LAYOUT_FIELD_INITIALIZED(T, Name, Value, ...) UE_REMOVE_OPTIONAL_PARENS(T) Name = Value; INTERNAL_LAYOUT_FIELD(T, Name, STRUCT_OFFSET(DerivedType, Name), EFieldLayoutFlags::MakeFlags(__VA_ARGS__), 1u, 0u, __COUNTER__)
#define LAYOUT_MUTABLE_FIELD_INITIALIZED(T, Name, Value, ...) mutable UE_REMOVE_OPTIONAL_PARENS(T) Name = Value; INTERNAL_LAYOUT_FIELD(T, Name, STRUCT_OFFSET(DerivedType, Name), EFieldLayoutFlags::MakeFlags(__VA_ARGS__), 1u, 0u, __COUNTER__)
#define LAYOUT_ARRAY(T, Name, NumArray, ...) UE_REMOVE_OPTIONAL_PARENS(T) Name[NumArray]; INTERNAL_LAYOUT_FIELD(T, Name, STRUCT_OFFSET(DerivedType, Name), EFieldLayoutFlags::MakeFlags(__VA_ARGS__), NumArray, 0u, __COUNTER__)
#define LAYOUT_MUTABLE_BITFIELD(T, Name, BitFieldSize, ...) mutable UE_REMOVE_OPTIONAL_PARENS(T) Name : BitFieldSize; INTERNAL_LAYOUT_FIELD(T, Name, ~0u, EFieldLayoutFlags::MakeFlags(__VA_ARGS__), 1u, BitFieldSize, __COUNTER__)
#define LAYOUT_BITFIELD(T, Name, BitFieldSize, ...) UE_REMOVE_OPTIONAL_PARENS(T) Name : BitFieldSize; INTERNAL_LAYOUT_FIELD(T, Name, ~0u, EFieldLayoutFlags::MakeFlags(__VA_ARGS__), 1u, BitFieldSize, __COUNTER__)
#define LAYOUT_FIELD_WITH_WRITER(T, Name, Func) UE_REMOVE_OPTIONAL_PARENS(T) Name; INTERNAL_LAYOUT_FIELD_WITH_WRITER(T, Name, Func, __COUNTER__)
#define LAYOUT_MUTABLE_FIELD_WITH_WRITER(T, Name, Func) mutable UE_REMOVE_OPTIONAL_PARENS(T) Name; INTERNAL_LAYOUT_FIELD_WITH_WRITER(T, Name, Func, __COUNTER__)
#define LAYOUT_WRITE_MEMORY_IMAGE(Func) INTERNAL_LAYOUT_WRITE_MEMORY_IMAGE(Func, __COUNTER__)
#define LAYOUT_TOSTRING(Func) INTERNAL_LAYOUT_TOSTRING(Func, __COUNTER__)

#if WITH_EDITORONLY_DATA
#define LAYOUT_FIELD_EDITORONLY(T, Name, ...) UE_REMOVE_OPTIONAL_PARENS(T) Name; INTERNAL_LAYOUT_FIELD(T, Name, STRUCT_OFFSET(DerivedType, Name), EFieldLayoutFlags::MakeFlagsEditorOnly(__VA_ARGS__), 1u, 0u, __COUNTER__)
#define LAYOUT_ARRAY_EDITORONLY(T, Name, NumArray, ...) UE_REMOVE_OPTIONAL_PARENS(T) Name[NumArray]; INTERNAL_LAYOUT_FIELD(T, Name, STRUCT_OFFSET(DerivedType, Name), EFieldLayoutFlags::MakeFlagsEditorOnly(__VA_ARGS__), NumArray, 0u, __COUNTER__)
#define LAYOUT_BITFIELD_EDITORONLY(T, Name, BitFieldSize, ...) UE_REMOVE_OPTIONAL_PARENS(T) Name : BitFieldSize; INTERNAL_LAYOUT_FIELD(T, Name, ~0u, EFieldLayoutFlags::MakeFlagsEditorOnly(__VA_ARGS__), 1u, BitFieldSize, __COUNTER__)
#else
#define LAYOUT_FIELD_EDITORONLY(T, Name, ...)
#define LAYOUT_ARRAY_EDITORONLY(T, Name, NumArray, ...)
#define LAYOUT_BITFIELD_EDITORONLY(T, Name, BitFieldSize, ...)
#endif

#define INTERNAL_LAYOUT_INTERFACE_PREFIX_NonVirtual(...) __VA_ARGS__
#define INTERNAL_LAYOUT_INTERFACE_PREFIX_Virtual(...) __VA_ARGS__ virtual
#define INTERNAL_LAYOUT_INTERFACE_PREFIX_Abstract(...) virtual
#define INTERNAL_LAYOUT_INTERFACE_PREFIX(Type) UE_JOIN(INTERNAL_LAYOUT_INTERFACE_PREFIX_, Type)

#define INTERNAL_LAYOUT_INTERFACE_SUFFIX_NonVirtual ;
#define INTERNAL_LAYOUT_INTERFACE_SUFFIX_Virtual ;
#define INTERNAL_LAYOUT_INTERFACE_SUFFIX_Abstract { return FTypeLayoutDesc::GetInvalidTypeLayout(); }
#define INTERNAL_LAYOUT_INTERFACE_SUFFIX(Type) UE_JOIN(INTERNAL_LAYOUT_INTERFACE_SUFFIX_, Type)

#define INTERNAL_LAYOUT_INTERFACE_INLINE_IMPL_NonVirtual { return StaticGetTypeLayout(); }
#define INTERNAL_LAYOUT_INTERFACE_INLINE_IMPL_Virtual { return StaticGetTypeLayout(); }
#define INTERNAL_LAYOUT_INTERFACE_INLINE_IMPL_Abstract { return FTypeLayoutDesc::GetInvalidTypeLayout(); }
#define INTERNAL_LAYOUT_INTERFACE_INLINE_IMPL(Type) UE_JOIN(INTERNAL_LAYOUT_INTERFACE_INLINE_IMPL_, Type)

#define INTERNAL_DECLARE_TYPE_LAYOUT_COMMON(T, InInterface) \
	static constexpr int CounterBase = __COUNTER__; \
	public: using DerivedType = UE_REMOVE_OPTIONAL_PARENS(T); \
	static constexpr ETypeLayoutInterface::Type InterfaceType = ETypeLayoutInterface::InInterface; \
	UE_DECLARE_INTERNAL_LINK_BASE(InternalLinkType) { UE_STATIC_ONLY(InternalLinkType); static UE_FORCEINLINE_HINT void Initialize(FTypeLayoutDesc& TypeDesc) {} }

#define INTERNAL_DECLARE_INLINE_TYPE_LAYOUT(T, InInterface) \
	private: static void InternalDestroy(void* Object, const FTypeLayoutDesc&, const FPointerTableBase* PtrTable, bool bIsFrozen) { \
		Freeze::DestroyObject(static_cast<UE_REMOVE_OPTIONAL_PARENS(T)*>(Object), PtrTable, bIsFrozen); \
	} \
	public: static FTypeLayoutDesc& StaticGetTypeLayout() { \
		static_assert(TValidateInterfaceHelper<UE_REMOVE_OPTIONAL_PARENS(T), ETypeLayoutInterface::InInterface>::Value, #InInterface " is invalid interface for " #T); \
		static FTypeLayoutDesc& StaticTypeDesc = []() -> FTypeLayoutDesc& \
		{ \
			alignas(FTypeLayoutDesc) static uint8 TypeBuffer[sizeof(FTypeLayoutDesc)] = { 0 }; \
			FTypeLayoutDesc& TypeDesc = *(FTypeLayoutDesc*)TypeBuffer; \
			TypeDesc.Name = TEXT(#T); \
			TypeDesc.WriteFrozenMemoryImageFunc = TGetFreezeImageHelper<UE_REMOVE_OPTIONAL_PARENS(T)>::Do(); \
			TypeDesc.UnfrozenCopyFunc = &Freeze::DefaultUnfrozenCopy; \
			TypeDesc.AppendHashFunc = &Freeze::DefaultAppendHash; \
			TypeDesc.GetTargetAlignmentFunc = &Freeze::DefaultGetTargetAlignment; \
			TypeDesc.ToStringFunc = &Freeze::DefaultToString; \
			TypeDesc.DestroyFunc = &InternalDestroy; \
			TypeDesc.Size = sizeof(UE_REMOVE_OPTIONAL_PARENS(T)); \
			TypeDesc.Alignment = alignof(UE_REMOVE_OPTIONAL_PARENS(T)); \
			TypeDesc.Interface = ETypeLayoutInterface::InInterface; \
			TypeDesc.SizeFromFields = ~0u; \
			TypeDesc.GetDefaultObjectFunc = &TGetDefaultObjectHelper<UE_REMOVE_OPTIONAL_PARENS(T), ETypeLayoutInterface::InInterface>::Do; \
			InternalLinkType<1>::Initialize(TypeDesc); \
			InternalInitializeBases<UE_REMOVE_OPTIONAL_PARENS(T)>(TypeDesc); \
			FTypeLayoutDesc::Initialize(TypeDesc); \
			return TypeDesc; \
		}(); \
		return StaticTypeDesc; } \
	public: INTERNAL_LAYOUT_INTERFACE_PREFIX(InInterface)(UE_EMPTY) const FTypeLayoutDesc& GetTypeLayout() const INTERNAL_LAYOUT_INTERFACE_INLINE_IMPL(InInterface) \
	INTERNAL_DECLARE_TYPE_LAYOUT_COMMON(T, InInterface)

#define INTERNAL_DECLARE_TYPE_LAYOUT(T, InInterface, RequiredAPI) \
	private: static void InternalDestroy(void* Object, const FTypeLayoutDesc&, const FPointerTableBase* PtrTable, bool bIsFrozen); \
	public: RequiredAPI static FTypeLayoutDesc& StaticGetTypeLayout(); \
	public: INTERNAL_LAYOUT_INTERFACE_PREFIX(InInterface)(RequiredAPI) const FTypeLayoutDesc& GetTypeLayout() const INTERNAL_LAYOUT_INTERFACE_SUFFIX(InInterface) \
	INTERNAL_DECLARE_TYPE_LAYOUT_COMMON(T, InInterface)

#define INTERNAL_DECLARE_LAYOUT_BASE(T) \
	private: using InternalBaseType = typename TGetBaseTypeHelper<UE_REMOVE_OPTIONAL_PARENS(T)>::Type; \
	template<typename InternalType> static void InternalInitializeBases(FTypeLayoutDesc& TypeDesc) { TInitializeBaseHelper<InternalType, InternalBaseType>::Initialize(TypeDesc); }

#define INTERNAL_DECLARE_LAYOUT_EXPLICIT_BASES(T, ...) \
	template<typename InternalType> static void InternalInitializeBases(FTypeLayoutDesc& TypeDesc) { InternalInitializeBasesHelper<InternalType, __VA_ARGS__>(TypeDesc); }

#define DECLARE_TYPE_LAYOUT(T, Interface) INTERNAL_DECLARE_LAYOUT_BASE(T); INTERNAL_DECLARE_TYPE_LAYOUT(T, Interface, UE_EMPTY)
#define DECLARE_INLINE_TYPE_LAYOUT(T, Interface) INTERNAL_DECLARE_LAYOUT_BASE(T); INTERNAL_DECLARE_INLINE_TYPE_LAYOUT(T, Interface)
#define DECLARE_EXPORTED_TYPE_LAYOUT(T, RequiredAPI, Interface) INTERNAL_DECLARE_LAYOUT_BASE(T); INTERNAL_DECLARE_TYPE_LAYOUT(T, Interface, RequiredAPI)

#define DECLARE_TYPE_LAYOUT_EXPLICIT_BASES(T, Interface, ...) INTERNAL_DECLARE_LAYOUT_EXPLICIT_BASES(T, __VA_ARGS__); INTERNAL_DECLARE_TYPE_LAYOUT(T, Interface, UE_EMPTY)
#define DECLARE_INLINE_TYPE_LAYOUT_EXPLICIT_BASES(T, Interface, ...) INTERNAL_DECLARE_LAYOUT_EXPLICIT_BASES(T, __VA_ARGS__); INTERNAL_DECLARE_INLINE_TYPE_LAYOUT(T, Interface)
#define DECLARE_EXPORTED_TYPE_LAYOUT_EXPLICIT_BASES(T, RequiredAPI, Interface, ...) INTERNAL_DECLARE_LAYOUT_EXPLICIT_BASES(T, __VA_ARGS__); INTERNAL_DECLARE_TYPE_LAYOUT(T, Interface, RequiredAPI)

#define INTERNAL_IMPLEMENT_TYPE_LAYOUT_COMMON(TemplatePrefix, T) \
	UE_REMOVE_OPTIONAL_PARENS(TemplatePrefix) void UE_REMOVE_OPTIONAL_PARENS(T)::InternalDestroy(void* Object, const FTypeLayoutDesc&, const FPointerTableBase* PtrTable, bool bIsFrozen) { \
		Freeze::DestroyObject(static_cast<UE_REMOVE_OPTIONAL_PARENS(T)*>(Object), PtrTable, bIsFrozen); \
	} \
	UE_REMOVE_OPTIONAL_PARENS(TemplatePrefix) FTypeLayoutDesc& UE_REMOVE_OPTIONAL_PARENS(T)::StaticGetTypeLayout() { \
		static_assert(TValidateInterfaceHelper<UE_REMOVE_OPTIONAL_PARENS(T), InterfaceType>::Value, "Invalid interface for " #T); \
		static FTypeLayoutDesc& StaticTypeDesc = []() -> FTypeLayoutDesc& \
		{ \
			alignas(FTypeLayoutDesc) static uint8 TypeBuffer[sizeof(FTypeLayoutDesc)] = { 0 }; \
			FTypeLayoutDesc& TypeDesc = *(FTypeLayoutDesc*)TypeBuffer; \
			TypeDesc.Name = TEXT(#T); \
			TypeDesc.WriteFrozenMemoryImageFunc = TGetFreezeImageHelper<UE_REMOVE_OPTIONAL_PARENS(T)>::Do(); \
			TypeDesc.UnfrozenCopyFunc = &Freeze::DefaultUnfrozenCopy; \
			TypeDesc.AppendHashFunc = &Freeze::DefaultAppendHash; \
			TypeDesc.GetTargetAlignmentFunc = &Freeze::DefaultGetTargetAlignment; \
			TypeDesc.ToStringFunc = &Freeze::DefaultToString; \
			TypeDesc.DestroyFunc = &InternalDestroy; \
			TypeDesc.Size = sizeof(UE_REMOVE_OPTIONAL_PARENS(T)); \
			TypeDesc.Alignment = alignof(UE_REMOVE_OPTIONAL_PARENS(T)); \
			TypeDesc.Interface = InterfaceType; \
			TypeDesc.SizeFromFields = ~0u; \
			TypeDesc.GetDefaultObjectFunc = &TGetDefaultObjectHelper<UE_REMOVE_OPTIONAL_PARENS(T), InterfaceType>::Do; \
			InternalLinkType<1>::Initialize(TypeDesc); \
			InternalInitializeBases<UE_REMOVE_OPTIONAL_PARENS(T)>(TypeDesc); \
			FTypeLayoutDesc::Initialize(TypeDesc); \
			return TypeDesc; \
		}(); \
		return StaticTypeDesc; }

//#define INTERNAL_REGISTER_TYPE_LAYOUT(T) static const FRegisterTypeLayoutDesc ANONYMOUS_VARIABLE(RegisterTypeLayoutDesc)(UE_REMOVE_OPTIONAL_PARENS(T)::StaticGetTypeLayout())
#define INTERNAL_REGISTER_TYPE_LAYOUT(T) static const FDelayedAutoRegisterHelper ANONYMOUS_VARIABLE(DelayedAutoRegisterHelper)(EDelayedRegisterRunPhase::ShaderTypesReady, []{ FTypeLayoutDesc::Register(T::StaticGetTypeLayout()); });

#define IMPLEMENT_UNREGISTERED_TEMPLATE_TYPE_LAYOUT(TemplatePrefix, T) \
	INTERNAL_IMPLEMENT_TYPE_LAYOUT_COMMON(TemplatePrefix, T); \
	UE_REMOVE_OPTIONAL_PARENS(TemplatePrefix) const FTypeLayoutDesc& UE_REMOVE_OPTIONAL_PARENS(T)::GetTypeLayout() const { return StaticGetTypeLayout(); }

#define IMPLEMENT_TEMPLATE_TYPE_LAYOUT(TemplatePrefix, T) \
	IMPLEMENT_UNREGISTERED_TEMPLATE_TYPE_LAYOUT(TemplatePrefix, T); \
	INTERNAL_REGISTER_TYPE_LAYOUT(T)

#define IMPLEMENT_TYPE_LAYOUT(T) IMPLEMENT_TEMPLATE_TYPE_LAYOUT(, T)

#define IMPLEMENT_ABSTRACT_TYPE_LAYOUT(T) \
	INTERNAL_IMPLEMENT_TYPE_LAYOUT_COMMON(, T); \
	INTERNAL_REGISTER_TYPE_LAYOUT(T)

#define REGISTER_INLINE_TYPE_LAYOUT(T) \
	static const FDelayedAutoRegisterHelper ANONYMOUS_VARIABLE(DelayedAutoRegisterHelper)(EDelayedRegisterRunPhase::ShaderTypesReady, [] \
		{ \
			UE_REMOVE_OPTIONAL_PARENS(T)::StaticGetTypeLayout().Name = TEXT(#T); \
			FTypeLayoutDesc::Register(UE_REMOVE_OPTIONAL_PARENS(T)::StaticGetTypeLayout()); \
		} \
	);

struct CTypeLayout
{
	template<typename T>
	auto Requires(const T&) -> decltype(T::StaticGetTypeLayout());
};

template<typename T>
struct THasTypeLayout
{
	UE_STATIC_ONLY(THasTypeLayout);
	static constexpr bool Value = TModels_V<CTypeLayout, T>;
};

template<typename T>
struct TStaticGetTypeLayoutHelper
{
	UE_STATIC_ONLY(TStaticGetTypeLayoutHelper);
	static const FTypeLayoutDesc& Do() { return T::StaticGetTypeLayout(); }
};

template<typename T>
struct TGetTypeLayoutHelper
{
	UE_STATIC_ONLY(TGetTypeLayoutHelper);
	static const FTypeLayoutDesc& Do(const T& Object) { return Object.GetTypeLayout(); }
};

template<typename T>
inline const FTypeLayoutDesc& StaticGetTypeLayoutDesc() { return TStaticGetTypeLayoutHelper<T>::Do(); }

template<typename T>
inline const FTypeLayoutDesc& GetTypeLayoutDesc(const FPointerTableBase*, const T& Object) { return TGetTypeLayoutHelper<T>::Do(Object); }

CORE_API extern void InternalDeleteObjectFromLayout(void* Object, const FTypeLayoutDesc& TypeDesc, const FPointerTableBase* PtrTable, bool bIsFrozen);

template<typename T>
inline void DeleteObjectFromLayout(T* Object, const FPointerTableBase* PtrTable = nullptr, bool bIsFrozen = false)
{
	check(Object);
	const FTypeLayoutDesc& TypeDesc = GetTypeLayoutDesc(PtrTable, *Object);
	InternalDeleteObjectFromLayout(Object, TypeDesc, PtrTable, bIsFrozen);
}

#define DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT(TemplatePrefix, T) \
	UE_REMOVE_OPTIONAL_PARENS(TemplatePrefix) struct THasTypeLayout<UE_REMOVE_OPTIONAL_PARENS(T)> { UE_STATIC_ONLY(THasTypeLayout); static const bool Value = true; }; \
	UE_REMOVE_OPTIONAL_PARENS(TemplatePrefix) struct TStaticGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(T)> { \
		UE_STATIC_ONLY(TStaticGetTypeLayoutHelper); \
		static void CallWriteMemoryImage(FMemoryImageWriter& Writer, const void* Object, const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& DerivedTypeDesc) { \
			Freeze::IntrinsicWriteMemoryImage(Writer, *static_cast<UE_REMOVE_OPTIONAL_PARENS(T) const*>(Object), TypeDesc); \
		} \
		static uint32 CallUnfrozenCopy(const FMemoryUnfreezeContent& Context, const void* Object, const FTypeLayoutDesc& TypeDesc, void* OutDst) { \
			return Freeze::IntrinsicUnfrozenCopy(Context, *static_cast<UE_REMOVE_OPTIONAL_PARENS(T) const*>(Object), OutDst); \
		} \
		static uint32 CallAppendHash(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher) { \
			return Freeze::IntrinsicAppendHash(static_cast<UE_REMOVE_OPTIONAL_PARENS(T) const*>(nullptr), TypeDesc, LayoutParams, Hasher); \
		} \
		static uint32 CallGetTargetAlignment(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams) { \
			return Freeze::IntrinsicGetTargetAlignment(static_cast<UE_REMOVE_OPTIONAL_PARENS(T) const*>(nullptr), TypeDesc, LayoutParams); \
		} \
		static void CallToString(const void* Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext) { \
			return Freeze::IntrinsicToString(*static_cast<UE_REMOVE_OPTIONAL_PARENS(T) const*>(Object), TypeDesc, LayoutParams, OutContext); \
		} \
		static void CallDestroy(void* Object, const FTypeLayoutDesc&, const FPointerTableBase* PtrTable, bool bIsFrozen) { \
			Freeze::DestroyObject(static_cast<UE_REMOVE_OPTIONAL_PARENS(T)*>(Object), PtrTable, bIsFrozen); \
		} \
		static const FTypeLayoutDesc& Do() { \
			static FTypeLayoutDesc& StaticTypeDesc = []() -> FTypeLayoutDesc& \
			{ \
				alignas(FTypeLayoutDesc) static uint8 TypeBuffer[sizeof(FTypeLayoutDesc)] = { 0 }; \
				FTypeLayoutDesc& TypeDesc = *(FTypeLayoutDesc*)TypeBuffer; \
				TypeDesc.IsIntrinsic = true; \
				TypeDesc.Name = TEXT(#T); \
				TypeDesc.WriteFrozenMemoryImageFunc = &CallWriteMemoryImage; \
				TypeDesc.UnfrozenCopyFunc = &CallUnfrozenCopy; \
				TypeDesc.AppendHashFunc = &CallAppendHash; \
				TypeDesc.GetTargetAlignmentFunc = &CallGetTargetAlignment; \
				TypeDesc.ToStringFunc = &CallToString; \
				TypeDesc.DestroyFunc = &CallDestroy; \
				TypeDesc.Size = sizeof(UE_REMOVE_OPTIONAL_PARENS(T)); \
				TypeDesc.Alignment = alignof(UE_REMOVE_OPTIONAL_PARENS(T)); \
				TypeDesc.Interface = ETypeLayoutInterface::NonVirtual; \
				TypeDesc.SizeFromFields = sizeof(UE_REMOVE_OPTIONAL_PARENS(T)); \
				return TypeDesc; \
			}(); \
			return StaticTypeDesc; } }; \
	UE_REMOVE_OPTIONAL_PARENS(TemplatePrefix) struct TGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(T)> { \
		UE_STATIC_ONLY(TGetTypeLayoutHelper); \
		static const FTypeLayoutDesc& Do(const UE_REMOVE_OPTIONAL_PARENS(T)&) { return TStaticGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(T)>::Do(); }}

#define DECLARE_EXPORTED_TEMPLATE_INTRINSIC_TYPE_LAYOUT(TemplatePrefix, T, RequiredAPI) \
	UE_REMOVE_OPTIONAL_PARENS(TemplatePrefix) struct THasTypeLayout<UE_REMOVE_OPTIONAL_PARENS(T)> { UE_STATIC_ONLY(THasTypeLayout); static const bool Value = true; }; \
	UE_REMOVE_OPTIONAL_PARENS(TemplatePrefix) struct TStaticGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(T)> { \
		UE_STATIC_ONLY(TStaticGetTypeLayoutHelper); \
		static void CallWriteMemoryImage(FMemoryImageWriter& Writer, const void* Object, const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& DerivedTypeDesc); \
		static uint32 CallUnfrozenCopy(const FMemoryUnfreezeContent& Context, const void* Object, const FTypeLayoutDesc& TypeDesc, void* OutDst); \
		static uint32 CallAppendHash(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher); \
		static uint32 CallGetTargetAlignment(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams); \
		static void CallToString(const void* Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext); \
		static void CallDestroy(void* Object, const FTypeLayoutDesc&, const FPointerTableBase* PtrTable, bool bIsFrozen); \
		RequiredAPI static const FTypeLayoutDesc& Do(); }; \
	UE_REMOVE_OPTIONAL_PARENS(TemplatePrefix) struct TGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(T)> { \
		UE_STATIC_ONLY(TGetTypeLayoutHelper); \
		static const FTypeLayoutDesc& Do(const UE_REMOVE_OPTIONAL_PARENS(T)&) { return TStaticGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(T)>::Do(); }}

#define IMPLEMENT_EXPORTED_INTRINSIC_TYPE_LAYOUT(T) \
		void TStaticGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(T)>::CallWriteMemoryImage(FMemoryImageWriter& Writer, const void* Object, const FTypeLayoutDesc& TypeDesc, const FTypeLayoutDesc& DerivedTypeDesc) { \
			Freeze::IntrinsicWriteMemoryImage(Writer, *static_cast<UE_REMOVE_OPTIONAL_PARENS(T) const*>(Object), TypeDesc); \
		} \
		uint32 TStaticGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(T)>::CallUnfrozenCopy(const FMemoryUnfreezeContent& Context, const void* Object, const FTypeLayoutDesc& TypeDesc, void* OutDst) { \
			return Freeze::IntrinsicUnfrozenCopy(Context, *static_cast<UE_REMOVE_OPTIONAL_PARENS(T) const*>(Object), OutDst); \
		} \
		uint32 TStaticGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(T)>::CallAppendHash(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher) { \
			return Freeze::IntrinsicAppendHash(static_cast<UE_REMOVE_OPTIONAL_PARENS(T) const*>(nullptr), TypeDesc, LayoutParams, Hasher); \
		} \
		uint32 TStaticGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(T)>::CallGetTargetAlignment(const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams) { \
			return Freeze::IntrinsicGetTargetAlignment(static_cast<UE_REMOVE_OPTIONAL_PARENS(T) const*>(nullptr), TypeDesc, LayoutParams); \
		} \
		void TStaticGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(T)>::CallToString(const void* Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext) { \
			return Freeze::IntrinsicToString(*static_cast<UE_REMOVE_OPTIONAL_PARENS(T) const*>(Object), TypeDesc, LayoutParams, OutContext); \
		} \
		void TStaticGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(T)>::CallDestroy(void* Object, const FTypeLayoutDesc&, const FPointerTableBase* PtrTable, bool bIsFrozen) { \
			Freeze::DestroyObject(static_cast<UE_REMOVE_OPTIONAL_PARENS(T)*>(Object), PtrTable, bIsFrozen); \
		} \
		const FTypeLayoutDesc& TStaticGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(T)>::Do() { \
			static FTypeLayoutDesc& StaticTypeDesc = []() -> FTypeLayoutDesc& \
			{ \
				alignas(FTypeLayoutDesc) static uint8 TypeBuffer[sizeof(FTypeLayoutDesc)] = { 0 }; \
				FTypeLayoutDesc& TypeDesc = *(FTypeLayoutDesc*)TypeBuffer; \
				TypeDesc.IsIntrinsic = true; \
				TypeDesc.Name = TEXT(#T); \
				TypeDesc.WriteFrozenMemoryImageFunc = &CallWriteMemoryImage; \
				TypeDesc.UnfrozenCopyFunc = &CallUnfrozenCopy; \
				TypeDesc.AppendHashFunc = &CallAppendHash; \
				TypeDesc.GetTargetAlignmentFunc = &CallGetTargetAlignment; \
				TypeDesc.ToStringFunc = &CallToString; \
				TypeDesc.DestroyFunc = &CallDestroy; \
				TypeDesc.Size = sizeof(UE_REMOVE_OPTIONAL_PARENS(T)); \
				TypeDesc.Alignment = alignof(UE_REMOVE_OPTIONAL_PARENS(T)); \
				TypeDesc.Interface = ETypeLayoutInterface::NonVirtual; \
				TypeDesc.SizeFromFields = sizeof(UE_REMOVE_OPTIONAL_PARENS(T)); \
				return TypeDesc; \
			}(); \
			return StaticTypeDesc; }

#define DECLARE_INTRINSIC_TYPE_LAYOUT(T) DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT(template<>, T)

#define ALIAS_TEMPLATE_TYPE_LAYOUT(TemplatePrefix, T, Alias) \
	UE_REMOVE_OPTIONAL_PARENS(TemplatePrefix) struct TStaticGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(T)> : public TStaticGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(Alias)> { UE_STATIC_ONLY(TStaticGetTypeLayoutHelper); }; \
	UE_REMOVE_OPTIONAL_PARENS(TemplatePrefix) struct TGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(T)> : public TGetTypeLayoutHelper<UE_REMOVE_OPTIONAL_PARENS(Alias)> { UE_STATIC_ONLY(TGetTypeLayoutHelper); }

#define ALIAS_TYPE_LAYOUT(Type, Alias) \
	static_assert(sizeof(UE_REMOVE_OPTIONAL_PARENS(Type)) == sizeof(UE_REMOVE_OPTIONAL_PARENS(Alias)), "Using a type alias but the sizes don't match!"); \
	ALIAS_TEMPLATE_TYPE_LAYOUT(template<>, Type, Alias)

DECLARE_INTRINSIC_TYPE_LAYOUT(char);
DECLARE_INTRINSIC_TYPE_LAYOUT(short);
DECLARE_INTRINSIC_TYPE_LAYOUT(int);
DECLARE_INTRINSIC_TYPE_LAYOUT(int8);
DECLARE_INTRINSIC_TYPE_LAYOUT(long);
DECLARE_INTRINSIC_TYPE_LAYOUT(long long);
DECLARE_INTRINSIC_TYPE_LAYOUT(unsigned char);
DECLARE_INTRINSIC_TYPE_LAYOUT(unsigned short);
DECLARE_INTRINSIC_TYPE_LAYOUT(unsigned int);
DECLARE_INTRINSIC_TYPE_LAYOUT(unsigned long);
DECLARE_INTRINSIC_TYPE_LAYOUT(unsigned long long);
DECLARE_INTRINSIC_TYPE_LAYOUT(bool);
DECLARE_INTRINSIC_TYPE_LAYOUT(float);
DECLARE_INTRINSIC_TYPE_LAYOUT(double);
DECLARE_INTRINSIC_TYPE_LAYOUT(wchar_t);
DECLARE_INTRINSIC_TYPE_LAYOUT(char16_t);
DECLARE_INTRINSIC_TYPE_LAYOUT(void*);
DECLARE_INTRINSIC_TYPE_LAYOUT(ETypeLayoutInterface::Type);

DECLARE_INTRINSIC_TYPE_LAYOUT(FThreadSafeCounter);
DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT(template<typename T>, TEnumAsByte<T>);

// Map 'const' types to non-const type
ALIAS_TEMPLATE_TYPE_LAYOUT(template<typename T>, const T, T);

// All raw pointer types map to void*, since they're all handled the same way
ALIAS_TEMPLATE_TYPE_LAYOUT(template<typename T>, T*, void*);

struct FPlatformTypeLayoutParameters
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FPlatformTypeLayoutParameters, CORE_API, NonVirtual);

	enum Flags
	{
		Flag_Initialized = (1 << 0),
		Flag_Is32Bit = (1 << 1),
		Flag_AlignBases = (1 << 2),
		Flag_WithEditorOnly = (1 << 3),
	};

	LAYOUT_FIELD_INITIALIZED(uint32, MaxFieldAlignment, 0xffffffff);
	LAYOUT_FIELD_INITIALIZED(uint32, Flags, 0u);

	inline bool IsInitialized() const { return (Flags & Flag_Initialized) != 0u; }
	inline bool Is32Bit() const { return (Flags & Flag_Is32Bit) != 0u; }
	inline bool HasAlignBases() const { return (Flags & Flag_AlignBases) != 0u; }
	inline bool WithEditorOnly() const { return (Flags & Flag_WithEditorOnly) != 0u; }

	// May need dedicated flag for this, if we need to support case-preserving names in non-editor builds
	inline bool WithCasePreservingFName() const { return WithEditorOnly(); }

	inline uint32 GetRawPointerSize() const { return Is32Bit() ? sizeof(uint32) : sizeof(uint64); }

	friend inline FArchive& operator<<(FArchive& Ar, FPlatformTypeLayoutParameters& Ref)
	{
		return Ref.Serialize(Ar);
	}

	friend inline bool operator==(const FPlatformTypeLayoutParameters& Lhs, const FPlatformTypeLayoutParameters& Rhs)
	{
		return Lhs.Flags == Rhs.Flags && Lhs.MaxFieldAlignment == Rhs.MaxFieldAlignment;
	}

	friend inline bool operator!=(const FPlatformTypeLayoutParameters& Lhs, const FPlatformTypeLayoutParameters& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}

	CORE_API bool IsCurrentPlatform() const;
	CORE_API void InitializeForArchive(FArchive& Ar);

	/** Initializes for the given platform, or for current platform if TargetPlatform is nullptr */
	CORE_API void InitializeForPlatform(const ITargetPlatform* TargetPlatform);

	CORE_API void InitializeForPlatform(const FString& PlatformName, bool bHasEditorOnlyData);
	CORE_API void InitializeForCurrent();
	CORE_API void InitializeForMSVC();
	CORE_API void InitializeForClang();

	/**
	 * This is used for serializing into/from the DDC
	 */
	CORE_API FArchive& Serialize(FArchive& Ar);

	/**
	 * Allow the layout parameters to modify the given DDC key string.
	 * Since layout parameters are part of e.g. material shadermap ID, they should result in two different DDC entries for two different IDs,
	 * even if binary layouts happen to be compatible.
	 */
	CORE_API void AppendKeyString(FString& KeyString) const;
	CORE_API void Append(FShaderKeyGenerator& KeyGen) const;

private:
	// Hidden friend for FShaderKeyGenerator Append function
	friend inline void Append(FShaderKeyGenerator& KeyGen, const FPlatformTypeLayoutParameters& Value)
	{
		Value.Append(KeyGen);
	}
#if WITH_EDITOR
	// Compact binary API with hidden friend operator<<
	CORE_API void Save(FCbWriter& Writer) const;
	bool TryLoad(FCbFieldView Field);
	friend inline FCbWriter& operator<<(FCbWriter& Writer, const FPlatformTypeLayoutParameters& Value)
	{
		Value.Save(Writer);
		return Writer;
	}
	friend CORE_API bool LoadFromCompactBinary(FCbFieldView Field, FPlatformTypeLayoutParameters& OutValue);
#endif
};
