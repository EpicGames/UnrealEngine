// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Memory/MemoryView.h"
#include "Containers/UnrealString.h"
#include "Serialization/MemoryArchive.h"
#include "Containers/ArrayView.h"

/**
 * Archive for reading arbitrary data from the specified memory location
 */
class FMemoryReader : public FMemoryArchive
{
public:
	/**
  	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const override
	{
		return TEXT("FMemoryReader");
	}

	virtual int64 TotalSize() override
	{
		return FMath::Min((int64)Bytes.Num(), LimitSize);
	}

	void Serialize( void* Data, int64 Num )
	{
		if (Num && !IsError())
		{
			// Only serialize if we have the requested amount of data
			if (Offset + Num <= TotalSize())
			{
				FMemory::Memcpy( Data, &Bytes[(int32)Offset], Num );
				Offset += Num;
			}
			else
			{
				SetError();
			}
		}
	}

	explicit FMemoryReader( const TArray<uint8>& InBytes, bool bIsPersistent = false )
		: Bytes    (InBytes)
		, LimitSize(INT64_MAX)
	{
		this->SetIsLoading(true);
		this->SetIsPersistent(bIsPersistent);
	}

	/** With this method it's possible to attach data behind some serialized data. */
	void SetLimitSize(int64 NewLimitSize)
	{
		LimitSize = NewLimitSize;
	}

private:
	const TArray<uint8>& Bytes;
	int64                LimitSize;
};

/**
 * Archive for reading arbitrary data from the specified memory view
 */
class FMemoryReaderView : public FMemoryArchive
{
public:
	/**
  	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const override
	{
		return TEXT("FMemoryReaderView");
	}

	virtual int64 TotalSize() override
	{
		return FMath::Min(static_cast<int64>(Bytes.GetSize()), LimitSize);
	}

	void Serialize(void* Data, int64 Num)
	{
		if (Num && !IsError())
		{
			// Only serialize if we have the requested amount of data
			if (Offset + Num <= TotalSize())
			{
				FMutableMemoryView(Data, static_cast<uint64>(Num)).CopyFrom(Bytes.Mid(Offset, Num));
				Offset += Num;
			}
			else
			{
				SetError();
			}
		}
	}

	template <typename InElementType, typename InSizeType>
	explicit FMemoryReaderView(TArrayView<InElementType, InSizeType>&& InBytes, bool bIsPersistent = false)
		: FMemoryReaderView(InBytes, bIsPersistent)
	{
	}

	explicit FMemoryReaderView(FMemoryView InBytes, bool bIsPersistent = false)
		: Bytes(InBytes)
		, LimitSize(INT64_MAX)
	{
		this->SetIsLoading(true);
		this->SetIsPersistent(bIsPersistent);
	}

	/** With this method it's possible to attach data behind some serialized data. */
	void SetLimitSize(int64 NewLimitSize)
	{
		LimitSize = NewLimitSize;
	}

private:
	FMemoryView          Bytes;
	int64                LimitSize;
};
