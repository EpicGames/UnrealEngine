// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Math/NumericLimits.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "Serialization/MemoryArchive.h"

/**
 * Archive for storing arbitrary data to the specified memory location
 */

template <typename ArrayAllocatorType>
class TMemoryWriterBase : public FMemoryArchive
{
	using IndexSizeType = typename ArrayAllocatorType::SizeType;
	static constexpr int32 IndexSize = sizeof(IndexSizeType) * 8;

public:
	TMemoryWriterBase( TArray<uint8, ArrayAllocatorType>& InBytes, bool bIsPersistent = false, bool bSetOffset = false, const FName InArchiveName = NAME_None )
	: FMemoryArchive()
	, Bytes(InBytes)
	, ArchiveName(InArchiveName)
	{
		this->SetIsSaving(true);
		this->SetIsPersistent(bIsPersistent);
		if (bSetOffset)
		{
			Offset = InBytes.Num();
		}
	}

	virtual void Serialize(void* Data, int64 Num) override
	{
		const int64 NumBytesToAdd = Offset + Num - Bytes.Num();
		if( NumBytesToAdd > 0 )
		{
			const int64 NewArrayCount = Bytes.Num() + NumBytesToAdd;
			if constexpr (IndexSize < 64)
			{
				constexpr IndexSizeType MaxValue = TNumericLimits<IndexSizeType>::Max();
				if (NewArrayCount >= MaxValue)
				{
					UE_LOG(LogSerialization, Fatal, TEXT("TMemoryWriter with IndexSize=%d does not support data larger than %" INT64_FMT " bytes. Archive name: %s."), IndexSize, (int64)MaxValue, *ArchiveName.ToString());
				}
			}

			Bytes.AddUninitialized((IndexSizeType)NumBytesToAdd);
		}

		check((Offset + Num) <= Bytes.Num());
		
		if( Num )
		{
			FMemory::Memcpy(&Bytes[(IndexSizeType)Offset], Data, Num);
			Offset += Num;
		}
	}
	/**
  	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const override
	{
		if (ArchiveName != NAME_None)
		{
			return ArchiveName.ToString();
		}

		return FString::Printf(TEXT("FMemoryWriter%d"), IndexSize);
	}

	int64 TotalSize() override
	{
		return Bytes.Num();
	}

protected:

	TArray<uint8, ArrayAllocatorType>&	Bytes;

	/** Archive name, used to debugging, by default set to NAME_None. */
	const FName ArchiveName;
};

template <int IndexSize>
using TMemoryWriter = TMemoryWriterBase<TSizedDefaultAllocator<IndexSize>>;

// FMemoryWriter and FMemoryWriter64 are implemented as derived classes rather than aliases
// so that forward declarations will work.

class FMemoryWriter : public TMemoryWriter<32>
{
	using Super = TMemoryWriter<32>;

public:
	using Super::Super;
};

class FMemoryWriter64 : public TMemoryWriter<64>
{
	using Super = TMemoryWriter<64>;

public:
	using Super::Super;
};
