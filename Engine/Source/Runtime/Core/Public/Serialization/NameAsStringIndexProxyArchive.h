// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CoreTypes.h"
#include "Serialization/ArchiveProxy.h"
#include "UObject/NameTypes.h"

class FArchive;

/**
 * Implements a proxy archive that serializes FNames as string data or an index (if the same name is repeated).
 */
struct FNameAsStringIndexProxyArchive : public FArchiveProxy
{
	/** When FName is first encountered, it is added to the table and saved as a string, otherwise, its index is written. Indices can be looked up from this TSet since it is not compacted. */
	TSet<FName> NamesSeenOnSave;

	/** Table of names that is populated as the archive is being loaded. */
	TArray<FName> NamesLoaded;

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InInnerArchive The inner archive to proxy.
	 */
	FNameAsStringIndexProxyArchive(FArchive& InInnerArchive)
		: FArchiveProxy(InInnerArchive)
	{ }

	/**
	 * Serialize the given FName as an FString or an index (if we encountered it again)
	 */
	CORE_API virtual FArchive& operator<<(FName& N);
};
