// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Serialization/ArchiveProxy.h"

/**
 * Implements a proxy archive that serializes FNames as string data.
 */
struct FNameAsStringProxyArchive : public FArchiveProxy
{
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InInnerArchive The inner archive to proxy.
	 */
	 FNameAsStringProxyArchive(FArchive& InInnerArchive)
		 :	FArchiveProxy(InInnerArchive)
	 { }

	 CORE_API ~FNameAsStringProxyArchive();

	 CORE_API virtual FArchive& operator<<(class FName& N);
};
