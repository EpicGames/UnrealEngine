// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringView.h"
#include "IO/IoDispatcher.h"
#include "IO/IoHash.h"
#include "Memory/CompositeBuffer.h"
#include "Misc/AssertionMacros.h"
#include "Misc/DateTime.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "Misc/SecureHash.h"
#include "Serialization/CompactBinary.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"

class FAssetRegistryState;
class FCbObject;
class FCbObjectView;
class FLargeMemoryWriter;
class ICookedPackageWriter;
class IPackageStoreWriter;
class UObject;
namespace UE::Cook { class ICookSandbox; }
namespace UE::Cook { class IDeterminismHelper; }
struct FPackageStoreEntryResource;
struct FSavePackageArgs;
struct FSavePackageResultStruct;


namespace UE::Cook::Artifact
{
	struct FUpdateOplogPackagesContext;

	enum EFieldStorage : uint8
	{
		/**
		* The bytes of the field will be stored in the compact binary of the package's root entry rather than as a separate attachment.
		* This is useful for fields used as filters since no additional attachment needs to be fetched, but it makes every read of the oplog more expensive.
		*/
		Inline,
		/**
		* The bytes of the field will be store in an attachment, and the field on the package's root entry will be an
		* ObjectAttachment - a 20byte pointer to the attachment that holds the bytes of the field.
		* This should be the default for most added fields, to keep the oplog entries small.
		*/
		Attachment,
	};
}

enum class EPackageWriterResult : uint8
{
	Success,
	Error,
	Timeout,
};

/** Interface for SavePackage to write packages to storage. */
class IPackageWriter
{
public:
	virtual ~IPackageWriter() = default;

	struct FCapabilities
	{
		/**
		 * Whether an entry should be created for each BulkData stored in the BulkData section
		 * This is necessary for some Writers that need to be able to load the BulkDatas individually.
		 * For other writers the extra regions are an unnecessary performance cost.
		 */
		bool bDeclareRegionForEachAdditionalFile = false;

		/**
		 * Applicable only to cook saves, and only to -diffonly saves; suppresses output and breakpoints for diffs
		 * in the header.
		 */
		bool bIgnoreHeaderDiffs = false;

		/**
		 * Applicable only to cook saves: True if the SavePackage call should write extra debug data for debugging
		 * cook determinism or incremental cook issues.
		 */
		bool bDeterminismDebug = false;
	};

	/** Return capabilities/settings this PackageWriter has/requires 
	  */
	virtual FCapabilities GetCapabilities() const
	{
		return FCapabilities();
	}

	// Events the PackageWriter receives
	struct FBeginPackageInfo
	{
		FName	PackageName;
		FString	LooseFilePath;
	};

	/** Mark the beginning of a package store transaction for the specified package

		This must be called before any data is produced for a given package
	  */
	virtual void BeginPackage(const FBeginPackageInfo& Info) = 0;

	struct FCommitAttachmentInfo
	{
		FName Key;
		FCbObject Value;
		UE::Cook::Artifact::EFieldStorage FieldStorage;
	};
	enum class EWriteOptions
	{
		None = 0,
		WritePackage = 0x01,
		WriteSidecars = 0x02,
		Write = WritePackage | WriteSidecars,
		ComputeHash = 0x04,
		SaveForDiff = 0x08,
	};
	enum class ECommitStatus
	{
		Success,
		Canceled,
		NothingToCook,
		Error,
		NotCommitted,
	};
	struct FCommitPackageInfo
	{
		FName PackageName;
		FIoHash PackageHash;
		TArray<FCommitAttachmentInfo> Attachments;
		ECommitStatus Status;
		EWriteOptions WriteOptions;
	};

	/** Finalize a package started with BeginPackage()
	  */
	virtual void CommitPackage(FCommitPackageInfo&& Info) = 0;

	struct FPackageInfo
	{
		/** Associated Package Name Entry from BeginPackage */
		FName		PackageName;
		FString		LooseFilePath;
		uint64		HeaderSize = 0;
		FIoChunkId	ChunkId = FIoChunkId::InvalidChunkId;
		uint16		MultiOutputIndex = 0;
	};

	/** Write package data (exports and serialized header)

		This may only be called after a BeginPackage() call has been made
		to signal the start of a package store transaction
	  */
	virtual void WritePackageData(const FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive, const TArray<FFileRegion>& FileRegions) = 0;

	struct FBulkDataInfo
	{
		enum EType
		{
			AppendToExports,
			BulkSegment,
			Mmap,
			Optional,
			NumTypes,
		};

		/** Associated Package Name Entry */
		FName		PackageName;
		EType		BulkDataType = BulkSegment;
		FString		LooseFilePath;
		FIoChunkId	ChunkId = FIoChunkId::InvalidChunkId;
		uint16		MultiOutputIndex = 0;
	};

	/** Write bulk data for the current package
	  */
	virtual void WriteBulkData(const FBulkDataInfo& Info, const FIoBuffer& BulkData, const TArray<FFileRegion>& FileRegions) = 0;

	struct FAdditionalFileInfo
	{
		/** Associated Package Name Entry */
		FName		PackageName;
		FString		Filename;
		FIoChunkId	ChunkId = FIoChunkId::InvalidChunkId;
		uint16		MultiOutputIndex = 0;
	};

	/** Write separate files written by UObjects during cooking via UObject::CookAdditionalFiles. */
	virtual void WriteAdditionalFile(const FAdditionalFileInfo& Info, const FIoBuffer& FileData) = 0;

	struct FLinkerAdditionalDataInfo
	{
		/** Associated Package Name Entry */
		FName	PackageName;
		uint16	MultiOutputIndex = 0;
	};
	/** Write separate data written by UObjects via FLinkerSave::AdditionalDataToAppend. */
	virtual void WriteLinkerAdditionalData(const FLinkerAdditionalDataInfo& Info, const FIoBuffer& Data, const TArray<FFileRegion>& FileRegions) = 0;

	/** Report the size of the Footer that is added after Exports and BulkData but before the PackageTrailer */ 
	virtual int64 GetExportsFooterSize()
	{
		return 0;
	}

	struct FPackageTrailerInfo
	{
		FName PackageName;
		uint16 MultiOutputIndex = 0;
	};
	/** Write the PackageTrailer, a separate segment for some bulkdata that is written the end of the file. */
	virtual void WritePackageTrailer(const FPackageTrailerInfo& Info, const FIoBuffer& Data) = 0;

	/** Create the FLargeMemoryWriter to which the Header and Exports are written during the save. */
	virtual TUniquePtr<FLargeMemoryWriter> CreateLinkerArchive(FName PackageName, UObject* Asset, uint16 MultiOutputIndex) = 0;

	/** Returns an archive to be used when serializing exports. */
	virtual TUniquePtr<FLargeMemoryWriter> CreateLinkerExportsArchive(FName PackageName, UObject* Asset, uint16 MultiOutputIndex) = 0;

	/**
	 * Overridden by PackageWriters that handle bDeterminismDebug=true. A system will call this function to register
	 * their callback class for adding determinism diagnostics for the given object to the package save.
	 */
	virtual void RegisterDeterminismHelper(UObject* SourceObject,
		const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper)
	{
	}

	/** Report whether PreSave was already called by the PackageWriter before the current UPackage::Save call. */
	virtual bool IsPreSaveCompleted() const
	{
		return false;
	}

	/** Downcast function for IPackageWriters that implement the ICookedPackageWriters inherited interface. */
	virtual ICookedPackageWriter* AsCookedPackageWriter()
	{
		return nullptr;
	}
};

ENUM_CLASS_FLAGS(IPackageWriter::EWriteOptions);

/** Struct containing hashes computed during cooked package writing. */
struct FPackageHashes : FRefCountedObject
{
	/** Hashes for each chunk saved by the package. */
	TMap<FIoChunkId, FIoHash> ChunkHashes;

	/** This is a hash representing the entire package. Not consistently computed across PackageWriters! */
	FMD5Hash PackageHash;

	/**
	 * A Future that is triggered after all packages have been stored on *this.
	 * Left as an invalid TFuture when hashes are not async; caller should check for IsValid before
	 * chaining with .Then or .Next.
	 */
	TFuture<int> CompletionFuture;
};

namespace UE::PackageWriter::Private
{

/**
 * Information passed from SavePackage into a PackageWriter during cooking for the calls to
 * BeginCacheForCookedPlatformData on the UObjects in the package.
 * This type is a cooker implementation detail and might be changed without deprecation in a future version.
 */
struct FBeginCacheForCookedPlatformDataInfo
{
	FName PackageName;
	const ITargetPlatform* TargetPlatform;
	TConstArrayView<UObject*> SaveableObjects;
	uint32 SaveFlags;
};

/**
 * The data needed to asynchronously write that was passed to a PackageWriter (.uasset, .uexp, .ubulk, any optional
 * and any additional), without reference back to other data on the Writer.
 * This type is a cooker implementation detail and might be changed without deprecation in a future version.
 */
struct FWriteFileData
{
	FString Filename;
	FCompositeBuffer Buffer;
	TArray<FFileRegion> Regions;
	bool bIsSidecar = false;
	/**
	 * True if file should be hashed and its hash accumulated into the total cookhash for the package writing the
	 * file.
	 */
	bool bContributeToHash = true;
	/**
	 * True only if the file is known to have a filename specific to the package saving the file, so that two
	 * packages cannot both try to write the same file. If it is true we can write the file directly on CookWorkers
	 * during MPCook without the possibility of two CookWorkers writing the file at the same time.
	 */
	bool bPackageSpecificFilename = false;
	FIoChunkId ChunkId = FIoChunkId::InvalidChunkId;
};

/**
 * Backpointer interface to the Cooker for functions that are implemented in Cooker code for PackageWriters used
 * by the cooker rather than CookedPackageWriter code, but that are triggered from CookedPackageWriter code.
 * This type is a cooker implementation detail and might be changed without deprecation in a future version.
 */
class ICookerInterface
{
public:
	virtual EPackageWriterResult CookerBeginCacheForCookedPlatformData(FBeginCacheForCookedPlatformDataInfo& Info) = 0;
	virtual void RegisterDeterminismHelper(ICookedPackageWriter* PackageWriter, UObject* SourceObject,
		const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper) = 0;
	virtual bool IsDeterminismDebug() const = 0;
	virtual void WriteFileOnCookDirector(const FWriteFileData& FileData, FMD5& AccumulatedHash,
		const TRefCountPtr<FPackageHashes>& PackageHashes, IPackageWriter::EWriteOptions WriteOptions) = 0;
	virtual UE::Cook::ICookSandbox* GetCookSandbox() = 0;

protected:
	~ICookerInterface() = default;
};

} // namespace UE::PackageWriter::Private

/** Interface for cooking that writes cooked packages to storage usable by the runtime game. */
class ICookedPackageWriter : public IPackageWriter
{
public:
	virtual ~ICookedPackageWriter() = default;

	/**
	 * Called by the Cooker for PackageWriters it is using. If provided, the interface must be used to handle the
	 * virtual functions of the same name on the ICookedPackageWriter.
	 */
	virtual void SetCooker(UE::PackageWriter::Private::ICookerInterface* CookerInterface) = 0;

	enum class EPackageHeaderFormat
	{
		PackageFileSummary,
		ZenPackageSummary
	};

	struct FCookCapabilities
	{
		/** Whether this writer implements -diffonly or other difftypes. */
		bool bDiffModeSupported = false;

		/** If true, the cooker will assume no packages are written and will skip writing non-package data. */
		bool bReadOnly = false;

		/**
		 * If true, the PackageWriter can override which packages are incrementally skipped in
		 * UpdatePackageModificationStatus, and the cooker will therefore avoid assumptions about what is
		 * incrementally skipped
		 */
		bool bOverridesPackageModificationStatus = false;

		/** If true, this writer can save and load extra data for each package across cooks. */
		bool bOplogAttachments = false;

		/** If true, this writer can ignore the path length limits of the host platform and handle paths of arbitrary length in BeginPackage */
		bool bIgnorePathLengthLimits = false;

		/** What header format is produced as output by this writer. */
		EPackageHeaderFormat HeaderFormat = EPackageHeaderFormat::PackageFileSummary;
	};

	/** Return cook capabilities/settings this PackageWriter has/requires
	  */
	virtual FCookCapabilities GetCookCapabilities() const
	{
		return FCookCapabilities();
	}

	/** Return the timestamp of the previous cook, or FDateTime::MaxValue to indicate previous cook should be assumed newer than any other cook data. */
	virtual FDateTime GetPreviousCookTime() const
	{
		return FDateTime::MaxValue();
	}

	virtual ICookedPackageWriter* AsCookedPackageWriter() override
	{
		return this;
	}

	struct FReferencedPluginsInfo
	{
		TSet<FString> ReferencedPlugins;
		bool bReferencesGame = false;
		bool bReferencesEngine = false;
	};

	struct FCookInfo
	{
		enum ECookMode
		{
			CookByTheBookMode,
			CookOnTheFlyMode
		};

		TOptional<FReferencedPluginsInfo> ReferencedPlugins;
		ECookMode CookMode = CookByTheBookMode;
		bool bFullBuild = true;
		bool bLegacyIterativeSharedBuild = false;
		bool bWorkerOnSharedSandbox = false;
	};

	/** Delete outdated cooked data, etc.
	  */
	virtual void Initialize(const FCookInfo& Info) = 0;

	/** Signal the start of a cooking pass

		Package data may only be produced after BeginCook() has been called and
		before EndCook() is called
	  */
	virtual void BeginCook(const FCookInfo& Info) = 0;

	/** Signal the end of a cooking pass.
	  */
	virtual void EndCook(const FCookInfo& Info) = 0;

	/**
	 * Called from UCookOnTheFlyServer::PopulateCookedPackages. Load the oplog of previously cooked packages, if not
	 * already done, and return the number of them.
	 */
	virtual void PopulateOplog(const FAssetRegistryState& PreviousState, int32& OutNumPackagesInOplog) = 0;
	/**
	 * Called during CookByTheBookFinished. Remove Ops from the Oplog that have not been referenced recently
	 * and should be removed to save space in the oplog and in artifacts that cache data about packages.
	 * 
	 * @param OplogContext Information that describes which packages were cooked or their previously cooked op
	 *        inspected (by e.g. incremental cook) in the current cook. The PackageWriter should update the
	 *        LastReferencedDate for these packages before checking for ops that have not recently been referenced.
	 */
	virtual void UpdateLastReferenceDateAndPruneStaleOps(
		UE::Cook::Artifact::FUpdateOplogPackagesContext& OplogContext) = 0;
	/**
	 * Called during CookByTheBookFinished, after UpdateLastReferenceDateAndPruneStaleOps. Return the list of all
	 * packages that have cooked data from the current or previous cooks. CookArtifacts use this to cull their data
	 * when previously cooked packages are pruned.
	 */
	virtual TArray<FName> GetOplogPackageNames() = 0;
	/**
	 * Returns an Attachment that was previously commited for the given PackageName.
	 * Returns an empty object if not found.
	 */
	virtual FCbObject GetOplogAttachment(FName PackageName, FUtf8StringView AttachmentKey) = 0;

	/**
	 * Callback is called for every cross product pair of PackageNames X AttachmentKeys. The Attachment object
	 * passed to the callback is empty if not found.
	 */
	virtual void GetOplogAttachments(TArrayView<FName> PackageNames,
		TArrayView<FUtf8StringView> AttachmentKeys,
		TUniqueFunction<void(FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)>&& Callback) = 0;

	/**
	 * Read Oplog attachments for the given PackageNames in the BaseGame oplog for the current DLC cook. Returns empty objects
	 * if current cook is not a DLC cook or the basegame oplog is unavailable.
	 * Callback is called for every cross product pair of PackageNames X AttachmentKeys. The Attachment object
	 * passed to the callback is empty if not found.
	 */
	virtual void GetBaseGameOplogAttachments(TArrayView<FName> PackageNames,
		TArrayView<FUtf8StringView> AttachmentKeys,
		TUniqueFunction<void(FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)>&& Callback) = 0;

	/**
	 * Report commit status for the given package. If cooking incrementally, this reports the status
	 * from a previous cook if it has not yet been committed in the current cook.
	 */
	virtual ECommitStatus GetCommitStatus(FName PackageName) = 0;

	/**
	 * Remove the given cooked package(s) from storage; they have been modified since the last cook.
	 */
	virtual void RemoveCookedPackages(TArrayView<const FName> PackageNamesToRemove) = 0;

	/**
	 * Remove all cooked packages from storage.
	 */
	virtual void RemoveCookedPackages() = 0;

	struct FUpdatePackageModifiedStatusContext
	{
		FName PackageName;
		bool bIncrementallyUnmodified = false;
		bool bPreviouslyCooked = false;
		bool bInOutShouldIncrementallySkip = false;
	};
	/**
	 * During incremental cooking, signal the cooked package has been checked for changes,
	 * and decide whether it should be incrementally skipped.
	 */
	virtual void UpdatePackageModifiedStatus(FUpdatePackageModifiedStatusContext& Context)
	{
	}

	struct FDeleteByFree
	{
		void operator()(void* Ptr) const
		{
			FMemory::Free(Ptr);
		}
	};

	struct FPreviousCookedBytesData
	{
		TUniquePtr<uint8, FDeleteByFree> Data;
		int64 Size;
		int64 HeaderSize;
		int64 StartOffset;
	};
	/** Load the bytes of the previously-cooked package, used for diffing */
	virtual bool GetPreviousCookedBytes(const FPackageInfo& Info, FPreviousCookedBytesData& OutData)
	{
		// The subclass must override this method if it returns bDiffModeSupported=true in GetCookCapabilities
		unimplemented();
		return false;
	}
	/** Append all data to the Exports archive that would normally be done in CommitPackage, used for diffing. */
	virtual void CompleteExportsArchiveForDiff(FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive)
	{
		// The subclass must override this method if it returns bDiffModeSupported=true in GetCookCapabilities
		unimplemented();
	}

	using FBeginCacheForCookedPlatformDataInfo = UE::PackageWriter::Private::FBeginCacheForCookedPlatformDataInfo;
	// Helper callback type for Writers that need to send the message on to UCookOnTheFlyServer
	UE_DEPRECATED(5.7, "Use ICookedPackageWriterCookerInterface.BeginCacheForCookedPlatformData instead.")
	typedef TUniqueFunction<EPackageWriterResult(FBeginCacheForCookedPlatformDataInfo& Info)> FBeginCacheCallback;
	UE_DEPRECATED(5.7, "Use ICookedPackageWriterCookerInterface.RegisterDeterminismHelper instead.")
	typedef TUniqueFunction<void(UObject* SourceObject,
		const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper)> FRegisterDeterminismHelperCallback;

	virtual EPackageWriterResult BeginCacheForCookedPlatformData(FBeginCacheForCookedPlatformDataInfo& Info)
	{
		unimplemented();
		return EPackageWriterResult::Error;
	}

	/** Modify the SaveArgs if required before the first Save. Used for diffing. */
	virtual void UpdateSaveArguments(FSavePackageArgs& SaveArgs)
	{
	}
	/** Report whether an additional save is needed and set up for it if so. Used for diffing. */
	virtual bool IsAnotherSaveNeeded(FSavePackageResultStruct& PreviousResult, FSavePackageArgs& SaveArgs)
	{
		return false;
	}
	/**
	 * Asynchronously create a CompactBinary Object message that replicates all of the package data from package save
	 * that is collected in memory and written at end of cook rather than being written to disk during package save.
	 * Used during MPCook to transfer this information from CookWorker to CookDirector. Called after CommitPackage,
	 * and only on CookWorkers.
	 */
	virtual TFuture<FCbObject> WriteMPCookMessageForPackage(FName PackageName) = 0;

	/** Read PackageData written by WriteMPCookMessageForPackage on a CookWorker. Called only on CookDirector. */
	virtual bool TryReadMPCookMessageForPackage(FName PackageName, FCbObjectView Message) = 0;

	/** Downcast function for ICookedPackageWriters that implement the IPackageStoreWriter inherited interface. */
	virtual IPackageStoreWriter* AsPackageStoreWriter()
	{
		return nullptr;
	}

	/** 
	*	Cooked package writers asynchronously hash the chunks for each package after CommitPackage. Once cooking has completed,
	*	use this to acquire the results. This is synced using void UPackage::WaitForAsyncFileWrites() - do not access
	*	the results before that completes. Non-const so that the cooking process can Move the map of hashes.
	*/
	virtual TMap<FName, TRefCountPtr<FPackageHashes>>& GetPackageHashes() = 0;

	/**
	* Store a value under a given key that can be read during staging and in future incremental cooks.
	* KeyName must not start with /, which identifies packages, and must not equal any existing special op names.
	*/
	virtual void AppendOp(FName KeyName, FCbObject Payload)
	{
	}
};

static inline const ANSICHAR* LexToString(IPackageWriter::FBulkDataInfo::EType Value)
{
	switch (Value)
	{
	case IPackageWriter::FBulkDataInfo::AppendToExports:
		return "AppendToExports";
	case IPackageWriter::FBulkDataInfo::BulkSegment:
		return "Standard";
	case IPackageWriter::FBulkDataInfo::Mmap:
		return "Mmap";
	case IPackageWriter::FBulkDataInfo::Optional:
		return "Optional";
	case IPackageWriter::FBulkDataInfo::NumTypes:
		return "NumTypes";
	default:
		checkNoEntry();
		return "Unknown";
	}

}
