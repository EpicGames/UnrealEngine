// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/StringBuilder.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"

class FSHAHash;
struct FShaderHash;
struct FBlake3Hash;
struct FGuid;

/**
 * Output class passed to Append functions for shader data. It receives Append calls for the input data for shader
 * compilation. These input data should trigger a recompile if they change, and they are therefore added into the
 * key used for storage of shader data in DDC and incremental cooks.
 * 
 * Known types are appended through FShaderKeyGenerator Append member functions.
 * For other types, the standard API (which is used by e.g. the append function for TArray) for types that can be
 * appended to FShaderKeyGenerator is the function in global namespace:
 *     void Append(FShaderKeyGenerator& KeyGen, const TypeName& Value);
 * 
 * The proper Append function can be called for any type using operator<<.
 */
class FShaderKeyGenerator
{
public:
	/** Constructor that writes the appended data to a hash function. */
	inline FShaderKeyGenerator(TUniqueFunction<void(const void* Data, uint64 Size)>&& InResultFunc);
	/** Constructor that writes the appended data to a long human-readable debug string. */
	inline FShaderKeyGenerator(FString& InResultString);
	inline ~FShaderKeyGenerator();

	/**
	 * True iff the KeyGen is writing to a hash function. Writing to a hash function also implies that debugtext and
	 * separators will be skipped in the output.
	 */
	inline bool IsBinary() const;
	/** True iff the KeyGen is writing to a human-readable debug string. */
	inline bool IsText() const;

	/** Must not be called unless IsBinary is true. Append data directly to the hash function. */
	inline void BinaryAppend(const void* Data, uint64 Size);
	/** Must not be called unless IsText is true. Return a modifiable reference to the string being written. */
	inline FString& TextGetResultString();

	/** Append arbitrary text to the output string or hash function. */
	inline void Append(FStringView Value);
	/**
	 * Append arbitrary text to the output string or hash function. Null-terminated character pointer.
	 * Null pointer is a noop.
	 */
	inline void Append(const TCHAR* Value);
	/** Convert the FName to text (case-sensitive) and append it to the output string or hash function. */
	inline void Append(FName Value);
	/** Appendf the integer to the output string or pass it as binary data to the hash function. */
	inline void Append(int64 Value);
	/** Appendf the integer to the output string or pass it as binary data to the hash function. */
	inline void Append(uint64 Value);
	/** Appendf the integer to the output string or pass it as binary data to the hash function. */
	inline void Append(int32 Value);
	/** Appendf the integer to the output string or pass it as binary data to the hash function. */
	inline void Append(uint32 Value);
	/** Appendf the integer to the output string, using %X format, or pass it as binary data to the hash function. */
	inline void AppendHex(uint32 Value);
	/** Appendf 0 or 1 to the output string or pass a 0 or 1 uint8 the hash function. */
	inline void AppendBoolInt(bool Value);
	/** Append Value to the output string (equivalent to LexToString) or pass it as binary data to the hash function. */
	CORE_API void Append(const FBlake3Hash& Value);
	/** Append Value to the output string (EGuidFormats::Digits) or pass it as binary data to the hash function. */
	CORE_API void Append(const FGuid& Value);
	/** Append Value to the output string (equivalent to LexToString) or pass it as binary data to the hash function. */
	CORE_API void Append(const FSHAHash& Value);
	/** Append Value to the output string (16-char hex) or pass it as binary data to the hash function. */
	CORE_API void Append(const FShaderHash& Value);

	/** Append arbitrary text to the output human-readable string. Noop if !IsText. */
	inline void AppendDebugText(FStringView Value);
	/** Append the separator character '_' to the output human-readable string. Noop if !IsText. */
	inline void AppendSeparator();

private:
	enum class EOutputType
	{
		Text,
		Binary,
	};

private:
	TUniqueFunction<void(const void* Data, uint64 Size)> ResultFunc;
	FString* ResultString = nullptr;
	EOutputType OutputType = EOutputType::Text;
};


///////////////////////////////////////////////////////
// Global functions taking FShaderKeyGenerator argument
///////////////////////////////////////////////////////

/** Append an ArrayView of handled types to the KeyGen. Calls operator<<(KeyGen, TypeName) on each element.*/
template <typename T>
void Append(FShaderKeyGenerator& KeyGen, TConstArrayView<T> Value);

/** Append an Array of handled types to the KeyGen. */
template <typename T>
void Append(FShaderKeyGenerator& KeyGen, const TArray<T>& Value);

/**
 * (FShaderKeyGenerator&)KeyGen << (const T&) Value;
 * 
 * Append the given Value to the KeyGen, using FShaderKeyGenerator::Append for known types, or
 * Append(FShaderKeyGenerator&, const T&) for unknown types.
 * Forward declare not possible, see implementation below.
 */
// template <typename T>
// FShaderKeyGenerator& operator<<(FShaderKeyGenerator& KeyGen, const T& Value);


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////

inline FShaderKeyGenerator::FShaderKeyGenerator(TUniqueFunction<void(const void* Data, uint64 Size)>&& InResultFunc)
	: ResultFunc(MoveTemp(InResultFunc))
	, OutputType(EOutputType::Binary)
{
}

inline FShaderKeyGenerator::FShaderKeyGenerator(FString& InResultString)
	: ResultString(&InResultString)
	, OutputType(EOutputType::Text)
{
}

// This destructor needs to be defined so that it can be called manually from union destructors.
inline FShaderKeyGenerator::~FShaderKeyGenerator() = default;

inline bool FShaderKeyGenerator::IsBinary() const
{
	return OutputType == EOutputType::Binary;
}

inline bool FShaderKeyGenerator::IsText() const
{
	return OutputType == EOutputType::Text;
}

inline void FShaderKeyGenerator::BinaryAppend(const void* Data, uint64 Size)
{
	check(IsBinary());
	ResultFunc(Data, Size);
}

inline FString& FShaderKeyGenerator::TextGetResultString()
{
	check(IsText());
	return *ResultString;
}

inline void FShaderKeyGenerator::Append(FStringView Value)
{
	switch (OutputType)
	{
	case EOutputType::Text:
		ResultString->Append(Value);
		return;
	case EOutputType::Binary:
		ResultFunc(Value.GetData(), Value.Len() * sizeof(Value.GetData()[0]));
		return;
	default:
		checkNoEntry();
	}
}

inline void FShaderKeyGenerator::Append(const TCHAR* Value)
{
	Append(FStringView(Value));
}

inline void FShaderKeyGenerator::Append(FName Value)
{
	switch (OutputType)
	{
	case EOutputType::Text:
		Value.AppendString(*ResultString);
		return;
	case EOutputType::Binary:
	{
		TStringBuilder<FName::StringBufferSize> Builder(InPlace, Value);
		ResultFunc(Builder.GetData(), Builder.Len() * sizeof(Builder.GetData()[0]));
		return;
	}
	default:
		checkNoEntry();
	}
}

inline void FShaderKeyGenerator::Append(int64 Value)
{
	switch (OutputType)
	{
	case EOutputType::Text:
		ResultString->Appendf(TEXT("%" INT64_FMT), Value);
		return;
	case EOutputType::Binary:
		ResultFunc(&Value, sizeof(Value));
		return;
	default:
		checkNoEntry();
	}
}

inline void FShaderKeyGenerator::Append(uint64 Value)
{
	switch (OutputType)
	{
	case EOutputType::Text:
		ResultString->Appendf(TEXT("%" UINT64_FMT), Value);
		return;
	case EOutputType::Binary:
		ResultFunc(&Value, sizeof(Value));
		return;
	default:
		checkNoEntry();
	}
}

inline void FShaderKeyGenerator::Append(int32 Value)
{
	switch (OutputType)
	{
	case EOutputType::Text:
		ResultString->Appendf(TEXT("%d"), Value);
		return;
	case EOutputType::Binary:
		ResultFunc(&Value, sizeof(Value));
		return;
	default:
		checkNoEntry();
	}
}

inline void FShaderKeyGenerator::Append(uint32 Value)
{
	switch (OutputType)
	{
	case EOutputType::Text:
		ResultString->Appendf(TEXT("%u"), Value);
		return;
	case EOutputType::Binary:
		ResultFunc(&Value, sizeof(Value));
		return;
	default:
		checkNoEntry();
	}
}

inline void FShaderKeyGenerator::AppendHex(uint32 Value)
{
	switch (OutputType)
	{
	case EOutputType::Text:
		ResultString->Appendf(TEXT("%X"), Value);
		return;
	case EOutputType::Binary:
		ResultFunc(&Value, sizeof(Value));
		return;
	default:
		checkNoEntry();
	}
}

inline void FShaderKeyGenerator::AppendBoolInt(bool Value)
{
	switch (OutputType)
	{
	case EOutputType::Text:
		ResultString->AppendChar(Value ? '1' : '0');
		return;
	case EOutputType::Binary:
	{
		uint8 SizedValue = Value ? 1 : 0;
		ResultFunc(&SizedValue, sizeof(SizedValue));
		return;
	}
	default:
		checkNoEntry();
	}
}

inline void FShaderKeyGenerator::AppendDebugText(FStringView Value)
{
	switch (OutputType)
	{
	case EOutputType::Text:
		ResultString->Append(Value);
		return;
	case EOutputType::Binary:
		// Binary output ignores DebugText
		return;
	default:
		checkNoEntry();
	}
}

inline void FShaderKeyGenerator::AppendSeparator()
{
	switch (OutputType)
	{
	case EOutputType::Text:
		ResultString->AppendChar('_');
		return;
	case EOutputType::Binary:
		// Binary output ignores DebugText; Separator is a type of DebugText
		return;
	default:
		checkNoEntry();
	}
}

template <typename T>
void Append(FShaderKeyGenerator& KeyGen, const TArray<T>& Value)
{
	Append(KeyGen, TConstArrayView<T>(Value));
}

template <typename T>
void Append(FShaderKeyGenerator& KeyGen, TConstArrayView<T> Value)
{
	for (const T& Element : Value)
	{
		KeyGen << Element;
	}
}

/**
 * Template override for struct used in std::enable_if to report whether a type is a FShaderKeyGenerator known type
 * with an Append member function. This template override provides the false value for non-known types.
 */
template <typename T, typename = int>
struct IsFShaderKeyGeneratorKnownType
 : std::false_type
{
};

/**
 * Template override for struct used in std::enable_if to report whether a type is a FShaderKeyGenerator known type
 * with an Append member function. This template override provides the true value for known types.
 */
template <typename T>
struct IsFShaderKeyGeneratorKnownType<T, decltype(std::declval<FShaderKeyGenerator>().Append(std::declval<T>()), 0)>
 : std::true_type
{
};

/** Template overide of FShaderKeyGenerator&& operator<<(FShaderKeyGenerator&, const T&), for known types. */
template <typename T>
typename std::enable_if<IsFShaderKeyGeneratorKnownType<T>::value, FShaderKeyGenerator&>::type
operator<<(FShaderKeyGenerator& KeyGen, const T& Value)
{
	KeyGen.Append(Value);
	return KeyGen;
}

/** Template overide of FShaderKeyGenerator&& operator<<(FShaderKeyGenerator&, const T&), for non-known types. */
template <typename T>
typename std::enable_if<!IsFShaderKeyGeneratorKnownType<T>::value, FShaderKeyGenerator&>::type
operator<<(FShaderKeyGenerator& KeyGen, const T& Value)
{
	Append(KeyGen, Value);
	return KeyGen;
}

