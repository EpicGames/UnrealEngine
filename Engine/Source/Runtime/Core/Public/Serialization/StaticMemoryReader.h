// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Serialization/MemoryArchive.h"

/**
 * Archive for reading arbitrary data from the specified memory location
 */
class FStaticMemoryReader final : public FMemoryArchive
{
public:
	/**
  	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const override
	{
		return TEXT("FStaticMemoryReader");
	}

	virtual int64 TotalSize() override
	{
		return NumBytes;
	}

	// returns the raw data ptr assigned to the memory reader.
	const uint8* GetData() const { return Bytes; }

	void Serialize( void* Data, int64 Num )
	{
		if (Num && !IsError())
		{
			// Only serialize if we have the requested amount of data
			if (Offset + Num <= TotalSize())
			{
				FMemory::Memcpy( Data, &Bytes[Offset], Num );
				Offset += Num;
			}
			else
			{
				SetError();
			}
		}
	}

	explicit FStaticMemoryReader(const uint8* InBytes, int64 InNumBytes)
		: Bytes   (InBytes)
		, NumBytes(InNumBytes)
	{
		this->SetIsLoading(true);
	}

private:
	const uint8* Bytes;
	int64        NumBytes;
};
