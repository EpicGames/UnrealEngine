// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreTypes.h"
#include "Formatters/BinaryArchiveFormatter.h"
#include "Misc/Build.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Serialization/StructuredArchiveDefines.h"
#include "Serialization/StructuredArchiveFormatter.h"
#include "Serialization/StructuredArchiveFwd.h"
#include "Serialization/StructuredArchiveNameHelpers.h"
#include "Serialization/StructuredArchiveSlotBase.h"
#include "Serialization/StructuredArchiveSlots.h"
#include "Templates/UniqueObj.h"

#if !WITH_TEXT_ARCHIVE_SUPPORT
	#define UE_API FORCEINLINE
#else
	#define UE_API CORE_API
#endif

class FStructuredArchive;

namespace UE::StructuredArchive::Private
{
	enum class EElementType : unsigned char
	{
		Root,
		Record,
		Array,
		Stream,
		Map,
		AttributedValue,
	};

	enum class EEnteringAttributeState
	{
		NotEnteringAttribute,
		EnteringAttribute,
	};

	FElementId GetCurrentSlotElementIdImpl(FStructuredArchive& Ar);
	FArchiveFormatterType& GetFormatterImpl(FStructuredArchive& Ar);
	}

/**
 * Manages the state of an underlying FStructuredArchiveFormatter, and provides a consistent API for reading and writing to a structured archive.
 * 
 * Both reading and writing to the archive are *forward only* from an interface point of view. There is no point at which it is possible to 
 * require seeking.
 */
class FStructuredArchive
{
	friend FStructuredArchiveSlot;
	friend FStructuredArchiveRecord;
	friend FStructuredArchiveArray;
	friend FStructuredArchiveStream;
	friend FStructuredArchiveMap;

public:
	using FSlot   = FStructuredArchiveSlot;
	using FRecord = FStructuredArchiveRecord;
	using FArray  = FStructuredArchiveArray;
	using FStream = FStructuredArchiveStream;
	using FMap    = FStructuredArchiveMap;

	/**
	 * Constructor.
	 *
	 * @param InFormatter Formatter for the archive data
	 */
	UE_API explicit FStructuredArchive(FArchiveFormatterType& InFormatter);
	
	/**
	 * Default destructor. Closes the archive.
	 */
	UE_API ~FStructuredArchive();

	/**
	 * Start writing to the archive, and gets an interface to the root slot.
	 */
	UE_API FStructuredArchiveSlot Open();

	/**
	 * Flushes any remaining scope to the underlying formatter and closes the archive.
	 */
	UE_API void Close();

	/**
	 * Gets the serialization context from the underlying archive.
	 */
	UE_FORCEINLINE_HINT FArchive& GetUnderlyingArchive() const
	{
		return Formatter.GetUnderlyingArchive();
	}

	/**
	 * Gets the archiving state.
	 */
	UE_FORCEINLINE_HINT const FArchiveState& GetArchiveState() const
	{
		return GetUnderlyingArchive().GetArchiveState();
	}

	FStructuredArchive(const FStructuredArchive&) = delete;
	FStructuredArchive& operator=(const FStructuredArchive&) = delete;

private:
	friend class FStructuredArchiveChildReader;
	friend UE::StructuredArchive::Private::FElementId UE::StructuredArchive::Private::GetCurrentSlotElementIdImpl(FStructuredArchive& Ar);
	friend FArchiveFormatterType& UE::StructuredArchive::Private::GetFormatterImpl(FStructuredArchive& Ar);

	/**
	* Reference to the formatter that actually writes out the data to the underlying archive
	*/
	FArchiveFormatterType& Formatter;

#if WITH_TEXT_ARCHIVE_SUPPORT
	/**
	 * Whether the formatter requires structural metadata. This allows optimizing the path for binary archives in editor builds.
	 */
	const bool bRequiresStructuralMetadata;

	struct FElement
	{
		UE::StructuredArchive::Private::FElementId Id;
		UE::StructuredArchive::Private::EElementType Type;

		FElement(UE::StructuredArchive::Private::FElementId InId, UE::StructuredArchive::Private::EElementType InType)
			: Id(InId)
			, Type(InType)
		{
		}
	};

	struct FIdGenerator
	{
		UE::StructuredArchive::Private::FElementId Generate()
		{
			return UE::StructuredArchive::Private::FElementId(NextId++);
		}

	private:
		uint32 NextId = 1;
	};

	/**
	 * The next element id to be assigned
	 */
	FIdGenerator ElementIdGenerator;

	/**
	 * The ID of the root element.
	 */
	UE::StructuredArchive::Private::FElementId RootElementId;

	/**
	 * The element ID assigned for the current slot. Slots are transient, and only exist as placeholders until something is written into them. This is reset to 0 when something is created in a slot, and the created item can assume the element id.
	 */
	UE::StructuredArchive::Private::FElementId CurrentSlotElementId;

	/**
	 * Tracks the current stack of objects being written. Used by SetScope() to ensure that scopes are always closed correctly in the underlying formatter,
	 * and to make sure that the archive is always written in a forwards-only way (ie. writing to an element id that is not in scope will assert)
	 */
	TArray<FElement, TNonRelocatableInlineAllocator<32>> CurrentScope;

#if DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
	struct FContainer;

	/**
	 * For arrays and maps, stores the loop counter and size of the container. Also stores key names for records and maps in builds with DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS enabled.
	 */
	TArray<TUniqueObj<FContainer>> CurrentContainer;
#endif

	/**
	 * Whether or not we've just entered an attribute
	 */
	UE::StructuredArchive::Private::EEnteringAttributeState CurrentEnteringAttributeState = UE::StructuredArchive::Private::EEnteringAttributeState::NotEnteringAttribute;

	/**
	 * Enters the current slot for serializing a value. Asserts if the archive is not in a state about to write to an empty-slot.
	 */
	CORE_API void EnterSlot(UE::StructuredArchive::Private::FSlotPosition Slot, bool bEnteringAttributedValue = false);

	/**
	 * Enters the current slot, adding an element onto the stack. Asserts if the archive is not in a state about to write to an empty-slot.
	 *
	 * @return  The depth of the newly-entered slot.
	 */
	CORE_API int32 EnterSlotAsType(UE::StructuredArchive::Private::FSlotPosition Slot, UE::StructuredArchive::Private::EElementType ElementType);

	/**
	 * Leaves slot at the top of the current scope
	 */
	CORE_API void LeaveSlot();

	/**
	 * Switches to the scope for the given slot.
	 */
	CORE_API void SetScope(UE::StructuredArchive::Private::FSlotPosition Slot);
#endif
};

#if !WITH_TEXT_ARCHIVE_SUPPORT

	UE_FORCEINLINE_HINT FStructuredArchive::FStructuredArchive(FArchiveFormatterType& InFormatter)
		: Formatter(InFormatter)
	{
	}

	UE_FORCEINLINE_HINT FStructuredArchive::~FStructuredArchive()
	{
	}

	UE_FORCEINLINE_HINT FStructuredArchiveSlot FStructuredArchive::Open()
	{
		return FStructuredArchiveSlot(FStructuredArchiveSlot::EPrivateToken{}, *this);
	}

	UE_FORCEINLINE_HINT void FStructuredArchive::Close()
	{
	}

#endif

namespace UE::StructuredArchive::Private
{
	UE_FORCEINLINE_HINT FArchive& GetUnderlyingArchiveImpl(FStructuredArchive& StructuredArchive)
	{
		return StructuredArchive.GetUnderlyingArchive();
	}

	UE_FORCEINLINE_HINT FArchiveState& GetUnderlyingArchiveStateImpl(FStructuredArchive& StructuredArchive)
	{
		return StructuredArchive.GetUnderlyingArchive().GetArchiveState();
	}

#if WITH_TEXT_ARCHIVE_SUPPORT
	UE_FORCEINLINE_HINT FElementId GetCurrentSlotElementIdImpl(FStructuredArchive& StructuredArchive)
	{
		return StructuredArchive.CurrentSlotElementId;
	}
#endif

	UE_FORCEINLINE_HINT FArchiveFormatterType& GetFormatterImpl(FStructuredArchive& StructuredArchive)
	{
		return StructuredArchive.Formatter;
	}
}

#undef UE_API