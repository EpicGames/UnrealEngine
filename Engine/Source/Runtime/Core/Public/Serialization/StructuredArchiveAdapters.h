// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Concepts/Insertable.h"
#include "Templates/Models.h"
#include "Serialization/ArchiveProxy.h"
#include "Serialization/StructuredArchiveSlots.h"
#include "Templates/UniqueObj.h"

class FStructuredArchiveFromArchive
{
	UE_NONCOPYABLE(FStructuredArchiveFromArchive)

	static constexpr inline uint32 ImplSize = 400;
	static constexpr inline uint32 ImplAlignment = 8;

	struct FImpl;

public:
	CORE_API explicit FStructuredArchiveFromArchive(FArchive& Ar);
	CORE_API ~FStructuredArchiveFromArchive();

	CORE_API FStructuredArchiveSlot GetSlot();

private:
	// Implmented as a pimpl in order to reduce dependencies, but an inline one to avoid heap allocations
	alignas(ImplAlignment) uint8 ImplStorage[ImplSize];
};

#if WITH_TEXT_ARCHIVE_SUPPORT

class FArchiveFromStructuredArchiveImpl : public FArchiveProxy
{
	UE_NONCOPYABLE(FArchiveFromStructuredArchiveImpl)

		struct FImpl;

public:
	CORE_API explicit FArchiveFromStructuredArchiveImpl(FStructuredArchiveSlot Slot);
	CORE_API virtual ~FArchiveFromStructuredArchiveImpl();

	CORE_API virtual void Flush() override;
	CORE_API virtual bool Close() override;

	CORE_API virtual int64 Tell() override;
	CORE_API virtual int64 TotalSize() override;
	CORE_API virtual void Seek(int64 InPos) override;
	CORE_API virtual bool AtEnd() override;

	using FArchive::operator<<; // For visibility of the overloads we don't override

	//~ Begin FArchive Interface
	CORE_API virtual FArchive& operator<<(class FName& Value) override;
	CORE_API virtual FArchive& operator<<(class UObject*& Value) override;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	CORE_API virtual FArchive& operator<<(Verse::VCell*& Value) override;
#endif
	CORE_API virtual FArchive& operator<<(class FText& Value) override;
	//~ End FArchive Interface

	CORE_API virtual void Serialize(void* V, int64 Length) override;

	CORE_API virtual FArchive* GetCacheableArchive() override;

	CORE_API bool ContainsData() const;

protected:
	CORE_API virtual bool Finalize(FStructuredArchiveRecord Record);
	CORE_API void OpenArchive();

private:
	void Commit();

	// Implmented as a pimpl in order to reduce dependencies
	TUniqueObj<FImpl> Pimpl;
};

class FArchiveFromStructuredArchive
{
public:
	explicit FArchiveFromStructuredArchive(FStructuredArchiveSlot InSlot)
		: Impl(InSlot)
	{
	}

	FArchive& GetArchive() { return Impl; }
	const FArchive& GetArchive() const { return Impl; }

	void Close() { Impl.Close(); }

private:
	FArchiveFromStructuredArchiveImpl Impl;
};

#else

class FArchiveFromStructuredArchive
{
public:
	explicit FArchiveFromStructuredArchive(FStructuredArchiveSlot InSlot)
		: Ar(InSlot.GetUnderlyingArchive())
	{
	}

	FArchive& GetArchive() { return Ar; }
	const FArchive& GetArchive() const { return Ar; }

	void Close() {}

private:
	FArchive& Ar;
};

#endif

/**
 * Adapter operator which allows a type to stream to an FArchive when it already supports streaming to an FStructuredArchiveSlot.
 *
 * @param  Ar   The archive to read from or write to.
 * @param  Obj  The object to read or write.
 *
 * @return  A reference to the same archive as Ar.
 */
template <typename T,
	std::enable_if_t<!TModels_V<CInsertable<FArchive&>, T> && TModels_V<CInsertable<FStructuredArchiveSlot>, T>, int> = 0>
FArchive& operator<<(FArchive& Ar, T& Obj)
{
	FStructuredArchiveFromArchive ArAdapt(Ar);
	ArAdapt.GetSlot() << Obj;
	return Ar;
}

/**
 * Adapter operator which allows a type to stream to an FStructuredArchiveSlot when it already supports streaming to an FArchive.
 *
 * @param  Slot  The slot to read from or write to.
 * @param  Obj   The object to read or write.
 */
template <typename T,
	std::enable_if_t<TModels_V<CInsertable<FArchive&>, T> && !TModels_V<CInsertable<FStructuredArchiveSlot>, T>, int> = 0>
void operator<<(FStructuredArchiveSlot Slot, T& Obj)
{
#if WITH_TEXT_ARCHIVE_SUPPORT
	FArchiveFromStructuredArchive Adapter(Slot);
	FArchive& Ar = Adapter.GetArchive();
#else
	FArchive& Ar = Slot.GetUnderlyingArchive();
#endif
	Ar << Obj;
#if WITH_TEXT_ARCHIVE_SUPPORT
	Adapter.Close();
#endif
}

