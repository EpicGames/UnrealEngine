// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/Build.h"
#include "Misc/Optional.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveSlots.h"

#if !WITH_TEXT_ARCHIVE_SUPPORT
	#define UE_API FORCEINLINE
#else
	#define UE_API CORE_API
#endif

class FStructuredArchive;
class FStructuredArchiveFormatter;

/**
 * FStructuredArchiveChildReader
 *
 * Utility class for easily creating a structured archive that covers the data hierarchy underneath
 * the given slot
 *
 * Allows serialization code to get an archive instance for the current location, so that it can return to it
 * later on after the master archive has potentially moved on into a different location in the file.
 */
class FStructuredArchiveChildReader
{
public:
	UE_API explicit FStructuredArchiveChildReader(FStructuredArchiveSlot InSlot);
	UE_API ~FStructuredArchiveChildReader();

	// Non-copyable
	FStructuredArchiveChildReader(FStructuredArchiveChildReader&&) = delete;
	FStructuredArchiveChildReader(const FStructuredArchiveChildReader&) = delete;
	FStructuredArchiveChildReader& operator=(FStructuredArchiveChildReader&&) = delete;
	FStructuredArchiveChildReader& operator=(const FStructuredArchiveChildReader&) = delete;

	UE_FORCEINLINE_HINT FStructuredArchiveSlot GetRoot() const
	{
		return Root.GetValue();
	}

private:
	FStructuredArchiveFormatter* OwnedFormatter;
	FStructuredArchive* StructuredArchive;
	TOptional<FStructuredArchiveSlot> Root;
};

#if !WITH_TEXT_ARCHIVE_SUPPORT

inline FStructuredArchiveChildReader::FStructuredArchiveChildReader(FStructuredArchiveSlot InSlot)
	: OwnedFormatter(nullptr)
	, StructuredArchive(nullptr)
{
	StructuredArchive = new FStructuredArchive(InSlot.StructuredArchive.Formatter);
	Root.Emplace(StructuredArchive->Open());
}

UE_FORCEINLINE_HINT FStructuredArchiveChildReader::~FStructuredArchiveChildReader()
{
	delete StructuredArchive;
}

#endif

#undef UE_API