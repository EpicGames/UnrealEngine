// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

/**
 * DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS - if set, checks that nested container types are serialized correctly.
 */
#ifndef DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS
	#if DO_GUARD_SLOW
		#define DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS 1
	#else
		#define DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS 0
	#endif
#endif

/**
 * DO_STRUCTURED_ARCHIVE_UNIQUE_FIELD_NAME_CHECKS - if set, checks that field names are unique within a container.  Requires DO_STRUCTURED_ARCHIVE_CONTAINER_CHECKS.
 */
#ifndef DO_STRUCTURED_ARCHIVE_UNIQUE_FIELD_NAME_CHECKS
	#define DO_STRUCTURED_ARCHIVE_UNIQUE_FIELD_NAME_CHECKS 0
#endif
