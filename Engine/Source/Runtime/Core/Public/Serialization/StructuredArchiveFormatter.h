// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreFwd.h"
#include "CoreTypes.h"
#include "Serialization/StructuredArchiveNameHelpers.h"

class FArchive;
class FName;
class FString;
class FText;
class UObject;

/**
 * Specifies the type of a value in a slot. Used by FContextFreeArchiveFormatter for introspection.
 */
enum class EArchiveValueType
{
	None,
	Record,
	Array,
	Stream,
	Map,
	Int8,
	Int16,
	Int32,
	Int64,
	UInt8,
	UInt16,
	UInt32,
	UInt64,
	Float,
	Double,
	Bool,
	String,
	Name,
	Object,
	Text,
	WeakObjectPtr,
	SoftObjectPtr,
	SoftObjectPath,
	LazyObjectPtr,
	RawData,
	AttributedValue,
	Attribute,
};

/**
 * Interface to format data to and from an underlying archive. Methods on this class are validated to be correct
 * with the current archive state (eg. EnterObject/LeaveObject calls are checked to be matching), and do not need
 * to be validated by implementations.
 *
 * Any functions with the _TextOnly suffix are intended to be implemented when reading text archives that have
 * a fully defined document tree, and allow querying additional properties that aren't available when reading from
 * a pure binary archive. These functions will assert if called on a binary archive.
 */
class FStructuredArchiveFormatter
{
public:
	CORE_API virtual ~FStructuredArchiveFormatter();

	virtual FArchive& GetUnderlyingArchive() = 0;
	virtual FStructuredArchiveFormatter* CreateSubtreeReader() { return this; }

	virtual bool HasDocumentTree() const = 0;

	virtual void EnterRecord() = 0;
	virtual void LeaveRecord() = 0;
	virtual void EnterField(FArchiveFieldName Name) = 0;
	virtual void LeaveField() = 0;
	virtual bool TryEnterField(FArchiveFieldName Name, bool bEnterWhenWriting) = 0;

	virtual void EnterArray(int32& NumElements) = 0;
	virtual void LeaveArray() = 0;
	virtual void EnterArrayElement() = 0;
	virtual void LeaveArrayElement() = 0;

	virtual void EnterStream() = 0;
	virtual void LeaveStream() = 0;
	virtual void EnterStreamElement() = 0;
	virtual void LeaveStreamElement() = 0;

	virtual void EnterMap(int32& NumElements) = 0;
	virtual void LeaveMap() = 0;
	virtual void EnterMapElement(FString& Name) = 0;
	virtual void LeaveMapElement() = 0;

	virtual void EnterAttributedValue() = 0;
	virtual void EnterAttribute(FArchiveFieldName AttributeName) = 0;
	virtual void EnterAttributedValueValue() = 0;
	virtual void LeaveAttribute() = 0;
	virtual void LeaveAttributedValue() = 0;
	virtual bool TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenWriting) = 0;
	virtual bool TryEnterAttributedValueValue() = 0;

	virtual void Serialize(uint8& Value) = 0;
	virtual void Serialize(uint16& Value) = 0;
	virtual void Serialize(uint32& Value) = 0;
	virtual void Serialize(uint64& Value) = 0;
	virtual void Serialize(int8& Value) = 0;
	virtual void Serialize(int16& Value) = 0;
	virtual void Serialize(int32& Value) = 0;
	virtual void Serialize(int64& Value) = 0;
	virtual void Serialize(float& Value) = 0;
	virtual void Serialize(double& Value) = 0;
	virtual void Serialize(bool& Value) = 0;
	virtual void Serialize(UTF32CHAR& Value) = 0;
	virtual void Serialize(FString& Value) = 0;
	virtual void Serialize(FName& Value) = 0;
	virtual void Serialize(UObject*& Value) = 0;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual void Serialize(Verse::VCell*& Value) = 0;
#endif
	virtual void Serialize(FText& Value) = 0;
	virtual void Serialize(struct FWeakObjectPtr& Value) = 0;
	virtual void Serialize(struct FSoftObjectPtr& Value) = 0;
	virtual void Serialize(struct FSoftObjectPath& Value) = 0;
	virtual void Serialize(struct FLazyObjectPtr& Value) = 0;
	virtual void Serialize(struct FObjectPtr& Value) = 0;
	virtual void Serialize(TArray<uint8>& Value) = 0;
	virtual void Serialize(void* Data, uint64 DataSize) = 0;
};
