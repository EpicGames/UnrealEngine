// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_exports

#include "CoreTypes.h"
#include "Misc/Build.h"
#include <type_traits>

// Formatters
class FBinaryArchiveFormatter;
class FStructuredArchiveFormatter;

// Archives
class FStructuredArchive;
class FStructuredArchiveChildReader;

// Slots
class FStructuredArchiveSlot;
class FStructuredArchiveRecord;
class FStructuredArchiveArray;
class FStructuredArchiveStream;
class FStructuredArchiveMap;

/** Typedef for which formatter type to support */
/** Written using std::conditional_t to work around IncludeTool complaints about #if blocks in Fwd.h files */
using FArchiveFormatterType = std::conditional_t<WITH_TEXT_ARCHIVE_SUPPORT, FStructuredArchiveFormatter, FBinaryArchiveFormatter>;

// IWYU pragma: end_exports
