// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Identity.h"

// Define a structure to encapsulate a field name, which compiles to an empty object if WITH_TEXT_ARCHIVE_SUPPORT = 0
struct FArchiveFieldName
{
#if WITH_TEXT_ARCHIVE_SUPPORT
	const TCHAR* Name;
#endif

	FArchiveFieldName(const TCHAR* InName)
#if WITH_TEXT_ARCHIVE_SUPPORT
		: Name(InName)
#endif
	{
	}
};

namespace UE::StructuredArchive::Private
{
	/**
	 * Class to contain a named value for serialization. Intended to be created as a temporary and passed to object serialization methods.
	 */
	template <typename T>
	struct TNamedValue
	{
		FArchiveFieldName Name;
		T& Value;
	};

	/**
	 * Class to contain a named attribute for serialization. Intended to be created as a temporary and passed to object serialization methods.
	 */
	template <typename T>
	struct TNamedAttribute
	{
		FArchiveFieldName Name;
		T& Value;
	};

	/**
	 * Class to contain a named attribute for serialization, with a default. Intended to be created as a temporary and passed to object
	 * serialization methods, which can choose not to serialize the attribute if it matches the default.
	 */
	template <typename T>
	struct TOptionalNamedAttribute
	{
		FArchiveFieldName Name;
		T& Value;
		const T& Default;
	};

	template <typename T>
	UE_FORCEINLINE_HINT TNamedValue<T> MakeNamedValue(FArchiveFieldName Name, T& Value)
	{
		return TNamedValue<T>{ Name, Value };
	}

	template <typename T>
	UE_FORCEINLINE_HINT TNamedAttribute<T> MakeNamedAttribute(FArchiveFieldName Name, T& Value)
	{
		return TNamedAttribute<T>{ Name, Value };
	}

	template <typename T>
	UE_FORCEINLINE_HINT TOptionalNamedAttribute<T> MakeOptionalNamedAttribute(FArchiveFieldName Name, T& Value, const typename TIdentity<T>::Type& Default)
	{
		return TOptionalNamedAttribute<T>{ Name, Value, Default };
	}
}

/** Construct a TNamedValue given an ANSI string and value reference. */
#define SA_VALUE(Name, Value) UE::StructuredArchive::Private::MakeNamedValue(FArchiveFieldName(Name), Value)

/** Construct a TNamedAttribute given an ANSI string and value reference. */
#define SA_ATTRIBUTE(Name, Value) UE::StructuredArchive::Private::MakeNamedAttribute(FArchiveFieldName(Name), Value)

/** Construct a TOptionalNamedAttribute given an ANSI string and value reference. */
#define SA_OPTIONAL_ATTRIBUTE(Name, Value, Default) UE::StructuredArchive::Private::MakeOptionalNamedAttribute(FArchiveFieldName(Name), Value, Default)
