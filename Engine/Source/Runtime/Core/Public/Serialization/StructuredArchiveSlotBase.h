// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class FArchive;
class FStructuredArchive;
struct FArchiveState;

namespace UE::StructuredArchive::Private
{
	struct FElementId
	{
		FElementId() = default;

		explicit FElementId(uint32 InId)
			: Id(InId)
		{
		}

		bool IsValid() const
		{
			return Id != 0;
		}

		void Reset()
		{
			Id = 0;
		}

		bool operator==(const FElementId& Rhs) const
		{
			return Id == Rhs.Id;
		}

		bool operator!=(const FElementId& Rhs) const
		{
			return Id != Rhs.Id;
		}

	private:
		uint32 Id = 0;
	};

	// Represents a position of a slot within the hierarchy.
	class FSlotPosition
	{
		friend FStructuredArchive;

	public:
		int32 Depth;
		FElementId ElementId;

		inline explicit FSlotPosition(int32 InDepth, FElementId InElementId)
			: Depth(InDepth)
			, ElementId(InElementId)
		{
		}
	};

	// The base class of all slot types
	class FSlotBase
#if WITH_TEXT_ARCHIVE_SUPPORT
		: protected FSlotPosition
#endif
	{
		friend FStructuredArchive;

	protected:
		// Token which needs to be passed to the constructor, so only friends are able to do so, but
		// still allows emplacement without the container being a friend.
		enum class EPrivateToken {};

	public:
#if WITH_TEXT_ARCHIVE_SUPPORT
		inline explicit FSlotBase(EPrivateToken, FStructuredArchive& InStructuredArchive, int32 InDepth, FElementId InElementId)
			: FSlotPosition(InDepth, InElementId)
			, StructuredArchive(InStructuredArchive)
		{
		}
#else
		UE_FORCEINLINE_HINT explicit FSlotBase(EPrivateToken, FStructuredArchive& InStructuredArchive)
			: StructuredArchive(InStructuredArchive)
		{
		}
#endif

		CORE_API FArchive& GetUnderlyingArchive() const;
		CORE_API const FArchiveState& GetArchiveState() const;

	protected:
		FStructuredArchive& StructuredArchive;
	};
}
