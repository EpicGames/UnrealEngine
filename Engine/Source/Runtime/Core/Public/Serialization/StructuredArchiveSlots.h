// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Formatters/BinaryArchiveFormatter.h"
#include "Misc/Build.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchiveFwd.h"
#include "Serialization/StructuredArchiveNameHelpers.h"
#include "Serialization/StructuredArchiveSlotBase.h"
#include "Templates/EnableIf.h"
#include "Templates/IsEnumClass.h"

#if !WITH_TEXT_ARCHIVE_SUPPORT
	#define UE_API FORCEINLINE
#else
	#define UE_API CORE_API
#endif

class FName;
class FString;
class FStructuredArchive;
class FStructuredArchiveArray;
class FStructuredArchiveChildReader;
class FStructuredArchiveMap;
class FStructuredArchiveRecord;
class FStructuredArchiveSlot;
class FStructuredArchiveStream;
class FText;
class UObject;
struct FLazyObjectPtr;
struct FObjectPtr;
struct FSoftObjectPath;
struct FSoftObjectPtr;
struct FWeakObjectPtr;
template <class TEnum> class TEnumAsByte;

namespace UE::StructuredArchive::Private
{
	FArchiveFormatterType& GetFormatterImpl(FStructuredArchive& StructuredArchive);
}

/**
 * Contains a value in the archive; either a field or array/map element. A slot does not know it's name or location,
 * and can merely have a value serialized into it. That value may be a literal (eg. int, float) or compound object
 * (eg. object, array, map).
 */
class FStructuredArchiveSlot final : public UE::StructuredArchive::Private::FSlotBase
{
public:
	UE_API FStructuredArchiveRecord EnterRecord();
	UE_API FStructuredArchiveArray EnterArray(int32& Num);
	UE_API FStructuredArchiveStream EnterStream();
	UE_API FStructuredArchiveMap EnterMap(int32& Num);
	UE_API FStructuredArchiveSlot EnterAttribute(FArchiveFieldName AttributeName);
	UE_API TOptional<FStructuredArchiveSlot> TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenWriting);

	// We don't support chaining writes to a single slot, so this returns void.
	UE_API void operator << (uint8& Value);
	UE_API void operator << (uint16& Value);
	UE_API void operator << (uint32& Value);
	UE_API void operator << (uint64& Value);
	UE_API void operator << (int8& Value);
	UE_API void operator << (int16& Value);
	UE_API void operator << (int32& Value);
	UE_API void operator << (int64& Value);
	UE_API void operator << (float& Value);
	UE_API void operator << (double& Value);
	UE_API void operator << (bool& Value);
	UE_API void operator << (UTF32CHAR& Value);
	UE_API void operator << (FString& Value);
	UE_API void operator << (FName& Value);
	UE_API void operator << (UObject*& Value);
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	UE_API void operator << (Verse::VCell*& Value);
#endif
	UE_API void operator << (FText& Value);
	UE_API void operator << (FWeakObjectPtr& Value);
	UE_API void operator << (FSoftObjectPtr& Value);
	UE_API void operator << (FSoftObjectPath& Value);
	UE_API void operator << (FLazyObjectPtr& Value);
	UE_API void operator << (FObjectPtr& Value);

	template <typename T>
	inline void operator<<(TEnumAsByte<T>& Value)
	{
		uint8 Tmp = (uint8)Value.GetValue();
		*this << Tmp;
		Value = (T)Tmp;
	}

	template <
		typename EnumType,
		std::enable_if_t<TIsEnumClass<EnumType>::Value, int> = 0
	>
	UE_FORCEINLINE_HINT void operator<<(EnumType& Value)
	{
		*this << (__underlying_type(EnumType)&)Value;
	}

	template <typename T>
	UE_FORCEINLINE_HINT void operator<<(UE::StructuredArchive::Private::TNamedAttribute<T> Item)
	{
		EnterAttribute(Item.Name) << Item.Value;
	}

	template <typename T>
	inline void operator<<(UE::StructuredArchive::Private::TOptionalNamedAttribute<T> Item)
	{
		if (TOptional<FStructuredArchiveSlot> Attribute = TryEnterAttribute(Item.Name, Item.Value != Item.Default))
		{
			Attribute.GetValue() << Item.Value;
		}
		else
		{
			Item.Value = Item.Default;
		}
	}

	UE_API void Serialize(TArray<uint8>& Data);
	UE_API void Serialize(void* Data, uint64 DataSize);
	CORE_API bool IsFilled() const;

private:
	friend FStructuredArchive;
	friend FStructuredArchiveChildReader;
	friend FStructuredArchiveSlot;
	friend FStructuredArchiveRecord;
	friend FStructuredArchiveArray;
	friend FStructuredArchiveStream;
	friend FStructuredArchiveMap;

	using UE::StructuredArchive::Private::FSlotBase::FSlotBase;
};

/**
 * Represents a record in the structured archive. An object contains slots that are identified by FArchiveName,
 * which may be compiled out with binary-only archives.
 */
class FStructuredArchiveRecord final : public UE::StructuredArchive::Private::FSlotBase
{
public:
	UE_API FStructuredArchiveSlot EnterField(FArchiveFieldName Name);
	UE_API FStructuredArchiveRecord EnterRecord(FArchiveFieldName Name);
	UE_API FStructuredArchiveArray EnterArray(FArchiveFieldName Name, int32& Num);
	UE_API FStructuredArchiveStream EnterStream(FArchiveFieldName Name);
	UE_API FStructuredArchiveMap EnterMap(FArchiveFieldName Name, int32& Num);

	UE_API TOptional<FStructuredArchiveSlot> TryEnterField(FArchiveFieldName Name, bool bEnterForSaving);

	template<typename T> inline FStructuredArchiveRecord& operator<<(UE::StructuredArchive::Private::TNamedValue<T> Item)
	{
		EnterField(Item.Name) << Item.Value;
		return *this;
	}

private:
	friend FStructuredArchive;
	friend FStructuredArchiveSlot;

	using UE::StructuredArchive::Private::FSlotBase::FSlotBase;
};

/**
 * Represents an array in the structured archive. An object contains slots that are identified by a FArchiveFieldName,
 * which may be compiled out with binary-only archives.
 */
class FStructuredArchiveArray final : public UE::StructuredArchive::Private::FSlotBase
{
public:
	UE_API FStructuredArchiveSlot EnterElement();

	template<typename T> inline FStructuredArchiveArray& operator<<(T& Item)
	{
		EnterElement() << Item;
		return *this;
	}

private:
	friend FStructuredArchive;
	friend FStructuredArchiveSlot;

	using UE::StructuredArchive::Private::FSlotBase::FSlotBase;
};

/**
 * Represents an unsized sequence of slots in the structured archive (similar to an array, but without a known size).
 */
class FStructuredArchiveStream final : public UE::StructuredArchive::Private::FSlotBase
{
public:
	UE_API FStructuredArchiveSlot EnterElement();

	template<typename T> inline FStructuredArchiveStream& operator<<(T& Item)
	{
		EnterElement() << Item;
		return *this;
	}

private:
	friend FStructuredArchive;
	friend FStructuredArchiveSlot;

	using UE::StructuredArchive::Private::FSlotBase::FSlotBase;
};

/**
 * Represents a map in the structured archive. A map is similar to a record, but keys can be read back out from an archive.
 * (This is an important distinction for binary archives).
 */
class FStructuredArchiveMap final : public UE::StructuredArchive::Private::FSlotBase
{
public:
	UE_API FStructuredArchiveSlot EnterElement(FString& Name);

private:
	friend FStructuredArchive;
	friend FStructuredArchiveSlot;

	using UE::StructuredArchive::Private::FSlotBase::FSlotBase;
};

template <typename T>
inline void operator<<(FStructuredArchiveSlot Slot, TArray<T>& InArray)
{
	int32 NumElements = InArray.Num();
	FStructuredArchiveArray Array = Slot.EnterArray(NumElements);

	if (Slot.GetArchiveState().IsLoading())
	{
		InArray.SetNum(NumElements);
	}

	for (int32 ElementIndex = 0; ElementIndex < NumElements; ++ElementIndex)
	{
		FStructuredArchiveSlot ElementSlot = Array.EnterElement();
		ElementSlot << InArray[ElementIndex];
	}
}

template <>
UE_FORCEINLINE_HINT void operator<<(FStructuredArchiveSlot Slot, TArray<uint8>& InArray)
{
	Slot.Serialize(InArray);
}

#if !WITH_TEXT_ARCHIVE_SUPPORT
	//////////// FStructuredArchiveSlot ////////////
	UE_FORCEINLINE_HINT FStructuredArchiveRecord FStructuredArchiveSlot::EnterRecord()
	{
		return FStructuredArchiveRecord(FStructuredArchiveRecord::EPrivateToken{}, StructuredArchive);
	}

	inline FStructuredArchiveArray FStructuredArchiveSlot::EnterArray(int32& Num)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).EnterArray(Num);
		return FStructuredArchiveArray(FStructuredArchiveArray::EPrivateToken{}, StructuredArchive);
	}

	UE_FORCEINLINE_HINT FStructuredArchiveStream FStructuredArchiveSlot::EnterStream()
	{
		return FStructuredArchiveStream(FStructuredArchiveStream::EPrivateToken{}, StructuredArchive);
	}

	inline FStructuredArchiveMap FStructuredArchiveSlot::EnterMap(int32& Num)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).EnterMap(Num);
		return FStructuredArchiveMap(FStructuredArchiveMap::EPrivateToken{}, StructuredArchive);
	}

	inline FStructuredArchiveSlot FStructuredArchiveSlot::EnterAttribute(FArchiveFieldName FieldName)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).EnterAttribute(FieldName);
		return FStructuredArchiveSlot(FStructuredArchiveSlot::EPrivateToken{}, StructuredArchive);
	}

	inline TOptional<FStructuredArchiveSlot> FStructuredArchiveSlot::TryEnterAttribute(FArchiveFieldName FieldName, bool bEnterWhenWriting)
	{
		if (UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).TryEnterAttribute(FieldName, bEnterWhenWriting))
		{
			return TOptional<FStructuredArchiveSlot>(InPlace, FStructuredArchiveSlot::EPrivateToken{}, StructuredArchive);
		}
		else
		{
			return TOptional<FStructuredArchiveSlot>();
		}
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (uint8& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (uint16& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (uint32& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (UTF32CHAR& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (uint64& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (int8& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (int16& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (int32& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (int64& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (float& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (double& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (bool& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (FString& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (FName& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (UObject*& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (Verse::VCell*& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}
#endif

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (FText& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (FWeakObjectPtr& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (FSoftObjectPath& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (FSoftObjectPtr& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (FLazyObjectPtr& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::operator<< (FObjectPtr& Value)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Value);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::Serialize(TArray<uint8>& Data)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Data);
	}

	UE_FORCEINLINE_HINT void FStructuredArchiveSlot::Serialize(void* Data, uint64 DataSize)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).Serialize(Data, DataSize);
	}

	//////////// FStructuredArchiveRecord ////////////

	UE_FORCEINLINE_HINT FStructuredArchiveSlot FStructuredArchiveRecord::EnterField(FArchiveFieldName Name)
	{
		return FStructuredArchiveSlot(FStructuredArchiveSlot::EPrivateToken{}, StructuredArchive);
	}

	UE_FORCEINLINE_HINT FStructuredArchiveRecord FStructuredArchiveRecord::EnterRecord(FArchiveFieldName Name)
	{
		return EnterField(Name).EnterRecord();
	}

	UE_FORCEINLINE_HINT FStructuredArchiveArray FStructuredArchiveRecord::EnterArray(FArchiveFieldName Name, int32& Num)
	{
		return EnterField(Name).EnterArray(Num);
	}

	UE_FORCEINLINE_HINT FStructuredArchiveStream FStructuredArchiveRecord::EnterStream(FArchiveFieldName Name)
	{
		return EnterField(Name).EnterStream();
	}

	UE_FORCEINLINE_HINT FStructuredArchiveMap FStructuredArchiveRecord::EnterMap(FArchiveFieldName Name, int32& Num)
	{
		return EnterField(Name).EnterMap(Num);
	}

	inline TOptional<FStructuredArchiveSlot> FStructuredArchiveRecord::TryEnterField(FArchiveFieldName Name, bool bEnterWhenWriting)
	{
		if (UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).TryEnterField(Name, bEnterWhenWriting))
		{
			return TOptional<FStructuredArchiveSlot>(InPlace, FStructuredArchiveSlot(FStructuredArchiveSlot::EPrivateToken{}, StructuredArchive));
		}
		else
		{
			return TOptional<FStructuredArchiveSlot>();
		}
	}

	//////////// FStructuredArchiveArray ////////////

	UE_FORCEINLINE_HINT FStructuredArchiveSlot FStructuredArchiveArray::EnterElement()
	{
		return FStructuredArchiveSlot(FStructuredArchiveSlot::EPrivateToken{}, StructuredArchive);
	}

	//////////// FStructuredArchiveStream ////////////

	UE_FORCEINLINE_HINT FStructuredArchiveSlot FStructuredArchiveStream::EnterElement()
	{
		return FStructuredArchiveSlot(FStructuredArchiveSlot::EPrivateToken{}, StructuredArchive);
	}

	//////////// FStructuredArchiveMap ////////////

	inline FStructuredArchiveSlot FStructuredArchiveMap::EnterElement(FString& Name)
	{
		UE::StructuredArchive::Private::GetFormatterImpl(StructuredArchive).EnterMapElement(Name);
		return FStructuredArchiveSlot(FStructuredArchiveSlot::EPrivateToken{}, StructuredArchive);
	}

#endif

#undef UE_API