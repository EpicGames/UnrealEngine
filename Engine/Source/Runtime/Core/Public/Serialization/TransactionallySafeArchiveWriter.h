// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM.h"
#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Templates/UniquePtr.h"

/**
 * FTransactionallySafeArchiveWriter takes ownership of an existing FArchive and allows it to
 * be written to during a transaction. This works by deferring file writes into a memory buffer
 * when inside a transaction. When the transaction is committed, the buffer is written into the
 * passed-in FArchive.
 * 
 * Outside of a transaction, writes are passed through to the wrapped archive transparently.
 * 
 * This class only supports basic archive functionality like << and Serialize. Other operations
 * like Seek and Tell are not supported.
 */
class FTransactionallySafeArchiveWriter : public FArchive
{
	using DeferredWriteBuffer = TArray<uint8>;

public:
	FTransactionallySafeArchiveWriter(TUniquePtr<FArchive> Ar) : InnerArchive(MoveTemp(Ar))
	{
		check(InnerArchive);
		check(InnerArchive->IsSaving());
		SetIsSaving(true);
	}

	~FTransactionallySafeArchiveWriter()
	{
		if (InnerArchive && bRegisteredCommitHandler)
		{
			// Our wrapper is going away, so move everything into a lambda-capture for safekeeping.
			AutoRTFM::PopOnCommitHandler(this);
			AutoRTFM::PushOnCommitHandler(this, [Archive = InnerArchive.Release(), WriteBuffer = MoveTemp(DeferredWrites), bFlush = bFlushRequested]() mutable
			{
				DoDeferredWrites(*Archive, WriteBuffer, bFlush);
				delete Archive;
			});
		}
	}

	/** 
	 * If we are outside of a transaction, you can have your archive back.
	 * (It's dangerous to allow Release within a transaction since this opens the door to out-of-sequence writes.)
	 */
	TUniquePtr<FArchive> Release()
	{
		check(!AutoRTFM::IsTransactional());
		return MoveTemp(InnerArchive);
	}

	virtual FString GetArchiveName() const override
	{ 
		return InnerArchive->GetArchiveName();
	}

	virtual void Seek(int64 InPos) final
	{
		unimplemented();
	}

	virtual int64 Tell() final
	{
		unimplemented();
		return 0;
	}

	/** Like all writes, flushes also need to be deferred to commit time. */
	virtual void Flush() override
	{
		if (MaybeRegisterCommitHandler())
		{
			// Defer the flush until commit time.
			bFlushRequested = true;
		}
		else
		{
			// We aren't in a transaction and don't have a commit handler, so nothing needs to be done. Forward to the inner archive.
			InnerArchive->Flush();
		}
	}

	virtual void Serialize(void* Data, int64 Num) override
	{
		check(InnerArchive);

		if (MaybeRegisterCommitHandler())
		{
			// Defer serialization until commit time.
			DeferredWrites.Append(static_cast<uint8*>(Data), IntCastChecked<DeferredWriteBuffer::SizeType>(Num));
		}
		else
		{
			// We aren't in a transaction and don't have a commit handler, so nothing needs to be done. Forward to the inner archive.
			check(DeferredWrites.IsEmpty());
			InnerArchive->Serialize(Data, Num);
		}
	}

private:
	bool MaybeRegisterCommitHandler()
	{
		// Returns true if a commit handler is in use.
		if (bRegisteredCommitHandler)
		{
			// If we already have a registered commit handler, we are either within a transaction or are committing/aborting.
			check(!AutoRTFM::IsIdle());
			return true;
		}
		else if (AutoRTFM::IsTransactional())
		{
			// If we don't have a commit handler set up yet, but find ourselves running transactionally, set up the commit handler now.
			RegisterCommitHandler();
			return true;
		}
		// We don't need a commit handler and can forward on requests directly to the inner archive.
		return false;
	}

	void RegisterCommitHandler()
	{
		check(!bRegisteredCommitHandler);
		bRegisteredCommitHandler = true;

		AutoRTFM::PushOnCommitHandler(this, [this]
		{
			DoDeferredWrites(*InnerArchive, DeferredWrites, bFlushRequested);
			bRegisteredCommitHandler = false;
		});
	}

	static void DoDeferredWrites(FArchive& Archive, DeferredWriteBuffer& WriteBuffer, bool& bFlushRequested)
	{
		Archive.Serialize(WriteBuffer.GetData(), WriteBuffer.NumBytes());
		WriteBuffer.Reset();

		if (bFlushRequested)
		{
			Archive.Flush();
			bFlushRequested = false;
		}
	}

	TUniquePtr<FArchive> InnerArchive;
	DeferredWriteBuffer DeferredWrites;
	bool bRegisteredCommitHandler = false;
	bool bFlushRequested = false;
};
