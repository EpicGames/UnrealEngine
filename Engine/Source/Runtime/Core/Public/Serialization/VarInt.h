// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformMath.h"

// Variable-Length Integer Encoding
//
// ZigZag encoding is used to convert signed integers into unsigned integers in a way that allows
// integers with a small magnitude to have a smaller encoded representation.
//
// An unsigned integer is encoded into 1-9 bytes based on its magnitude. The first byte indicates
// how many additional bytes are used by the number of leading 1-bits that it has. The additional
// bytes are stored in big endian order, and the most significant bits of the value are stored in
// the remaining bits in the first byte. The encoding of the first byte allows the reader to skip
// over the encoded integer without consuming its bytes individually.
//
// Encoded unsigned integers sort the same in a byte-wise comparison as when their decoded values
// are compared. The same property does not hold for signed integers due to ZigZag encoding.
//
// 32-bit inputs encode to 1-5 bytes.
// 64-bit inputs encode to 1-9 bytes.
//
// 0x0000'0000'0000'0000 - 0x0000'0000'0000'007f : 0b0_______ 1 byte
// 0x0000'0000'0000'0080 - 0x0000'0000'0000'3fff : 0b10______ 2 bytes
// 0x0000'0000'0000'4000 - 0x0000'0000'001f'ffff : 0b110_____ 3 bytes
// 0x0000'0000'0020'0000 - 0x0000'0000'0fff'ffff : 0b1110____ 4 bytes
// 0x0000'0000'1000'0000 - 0x0000'0007'ffff'ffff : 0b11110___ 5 bytes
// 0x0000'0008'0000'0000 - 0x0000'03ff'ffff'ffff : 0b111110__ 6 bytes
// 0x0000'0400'0000'0000 - 0x0001'ffff'ffff'ffff : 0b1111110_ 7 bytes
// 0x0002'0000'0000'0000 - 0x00ff'ffff'ffff'ffff : 0b11111110 8 bytes
// 0x0100'0000'0000'0000 - 0xffff'ffff'ffff'ffff : 0b11111111 9 bytes
//
// Encoding Examples
//                -42 => ZigZag => 0x53 => 0x53
//                 42 => ZigZag => 0x54 => 0x54
//                0x1 => 0x01
//               0x12 => 0x12
//              0x123 => 0x81 0x23
//             0x1234 => 0x92 0x34
//            0x12345 => 0xc1 0x23 0x45
//           0x123456 => 0xd2 0x34 0x56
//          0x1234567 => 0xe1 0x23 0x45 0x67
//         0x12345678 => 0xf0 0x12 0x34 0x56 0x78
//        0x123456789 => 0xf1 0x23 0x45 0x67 0x89
//       0x123456789a => 0xf8 0x12 0x34 0x56 0x78 0x9a
//      0x123456789ab => 0xfb 0x23 0x45 0x67 0x89 0xab
//     0x123456789abc => 0xfc 0x12 0x34 0x56 0x78 0x9a 0xbc
//    0x123456789abcd => 0xfd 0x23 0x45 0x67 0x89 0xab 0xcd
//   0x123456789abcde => 0xfe 0x12 0x34 0x56 0x78 0x9a 0xbc 0xde
//  0x123456789abcdef => 0xff 0x01 0x23 0x45 0x67 0x89 0xab 0xcd 0xef
// 0x123456789abcdef0 => 0xff 0x12 0x34 0x56 0x78 0x9a 0xbc 0xde 0xf0

/**
 * Measure the length in bytes (1-9) of an encoded variable-length integer.
 *
 * @param InData A variable-length encoding of an (signed or unsigned) integer.
 * @return The number of bytes used to encode the integer, in the range 1-9.
 */
FORCEINLINE uint32 MeasureVarUInt(const void* InData)
{
	return FPlatformMath::CountLeadingZeros(uint8(~*static_cast<const uint8*>(InData))) - 23;
}

/** Measure the length in bytes (1-9) of an encoded variable-length integer. \see \ref MeasureVarUInt */
FORCEINLINE uint32 MeasureVarInt(const void* InData)
{
	return MeasureVarUInt(InData);
}

/** Measure the number of bytes (1-5) required to encode the 32-bit input. */
FORCEINLINE uint32 MeasureVarUInt(uint32 InValue)
{
	// Retaining this version with no branching until both have been profiled on ARM.
	// This is equivalent to Min(FloorLog2(N) / 7 + 1, 9)
	//return FPlatformMath::Min(((uint32(FPlatformMath::FloorLog2NonZero(InValue | 1)) * 37) >> 8) + 1, 9u);

	// This branching implementation is faster than branchless alternatives in real-world benchmarks.
	return (InValue < (uint32(1) << 28))
		? ((InValue < (uint32(1) << 14))
			? ((InValue < (uint32(1) <<  7)) ? 1 : 2)
			: ((InValue < (uint32(1) << 21)) ? 3 : 4))
		: 5;
}

/** Measure the number of bytes (1-9) required to encode the 64-bit input. */
FORCEINLINE uint32 MeasureVarUInt(uint64 InValue)
{
	// Retaining this version with no branching until both have been profiled on ARM.
	// This is equivalent to Min(FloorLog2(N) / 7 + 1, 9)
	//return FPlatformMath::Min(((uint32(FPlatformMath::FloorLog2NonZero_64(InValue | 1)) * 37) >> 8) + 1, 9u);

	// This branching implementation is faster than branchless alternatives in real-world benchmarks.
	return (InValue < (uint64(1) << 28))
		? ((InValue < (uint64(1) << 14))
			? ((InValue < (uint64(1) <<  7)) ? 1 : 2)
			: ((InValue < (uint64(1) << 21)) ? 3 : 4))
		: ((InValue < (uint64(1) << 42))
			? ((InValue < (uint64(1) << 35)) ? 5 : 6)
			: ((InValue < (uint64(1) << 49)) ? 7 : ((InValue < (uint64(1) << 56)) ? 8 : 9)));
}

/** Measure the number of bytes (1-5) required to encode the 32-bit input. \see \ref MeasureVarUInt */
FORCEINLINE uint32 MeasureVarInt(int32 InValue)
{
	return MeasureVarUInt(uint32((InValue >> 31) ^ (InValue << 1)));
}

/** Measure the number of bytes (1-9) required to encode the 64-bit input. \see \ref MeasureVarUInt */
FORCEINLINE uint32 MeasureVarInt(int64 InValue)
{
	return MeasureVarUInt(uint64((InValue >> 63) ^ (InValue << 1)));
}

/**
 * Read a variable-length unsigned integer.
 *
 * @param InData A variable-length encoding of an unsigned integer.
 * @param OutByteCount The number of bytes consumed from the input.
 * @return An unsigned integer.
 */
FORCEINLINE uint64 ReadVarUInt(const void* InData, uint32& OutByteCount)
{
	const uint32 ByteCount = MeasureVarUInt(InData);
	OutByteCount = ByteCount;

	const uint8* InBytes = static_cast<const uint8*>(InData);
	uint64 Value = *InBytes++ & uint8(0xff >> ByteCount);
	switch (ByteCount - 1)
	{
	case 8: Value <<= 8; Value |= *InBytes++;
	case 7: Value <<= 8; Value |= *InBytes++;
	case 6: Value <<= 8; Value |= *InBytes++;
	case 5: Value <<= 8; Value |= *InBytes++;
	case 4: Value <<= 8; Value |= *InBytes++;
	case 3: Value <<= 8; Value |= *InBytes++;
	case 2: Value <<= 8; Value |= *InBytes++;
	case 1: Value <<= 8; Value |= *InBytes++;
	default:
		return Value;
	}
}

/**
 * Read a variable-length signed integer.
 *
 * @param InData A variable-length encoding of a signed integer.
 * @param OutByteCount The number of bytes consumed from the input.
 * @return A signed integer.
 */
FORCEINLINE int64 ReadVarInt(const void* InData, uint32& OutByteCount)
{
	const uint64 Value = ReadVarUInt(InData, OutByteCount);
	return -int64(Value & 1) ^ int64(Value >> 1);
}

/**
 * Write a variable-length unsigned integer.
 *
 * @param InValue An unsigned integer to encode.
 * @param OutData A buffer of at least 5 bytes to write the output to.
 * @return The number of bytes used in the output.
 */
FORCEINLINE uint32 WriteVarUInt(uint32 InValue, void* OutData)
{
	const uint32 ByteCount = MeasureVarUInt(InValue);
	uint8* OutBytes = static_cast<uint8*>(OutData) + ByteCount - 1;
	switch (ByteCount - 1)
	{
	case 4: *OutBytes-- = uint8(InValue); InValue >>= 8;
	case 3: *OutBytes-- = uint8(InValue); InValue >>= 8;
	case 2: *OutBytes-- = uint8(InValue); InValue >>= 8;
	case 1: *OutBytes-- = uint8(InValue); InValue >>= 8;
	default: break;
	}
	*OutBytes = uint8(0xff << (9 - ByteCount)) | uint8(InValue);
	return ByteCount;
}

/**
 * Write a variable-length unsigned integer.
 *
 * @param InValue An unsigned integer to encode.
 * @param OutData A buffer of at least 9 bytes to write the output to.
 * @return The number of bytes used in the output.
 */
FORCEINLINE uint32 WriteVarUInt(uint64 InValue, void* OutData)
{
	const uint32 ByteCount = MeasureVarUInt(InValue);
	uint8* OutBytes = static_cast<uint8*>(OutData) + ByteCount - 1;
	switch (ByteCount - 1)
	{
	case 8: *OutBytes-- = uint8(InValue); InValue >>= 8;
	case 7: *OutBytes-- = uint8(InValue); InValue >>= 8;
	case 6: *OutBytes-- = uint8(InValue); InValue >>= 8;
	case 5: *OutBytes-- = uint8(InValue); InValue >>= 8;
	case 4: *OutBytes-- = uint8(InValue); InValue >>= 8;
	case 3: *OutBytes-- = uint8(InValue); InValue >>= 8;
	case 2: *OutBytes-- = uint8(InValue); InValue >>= 8;
	case 1: *OutBytes-- = uint8(InValue); InValue >>= 8;
	default: break;
	}
	*OutBytes = uint8(0xff << (9 - ByteCount)) | uint8(InValue);
	return ByteCount;
}

/** Write a variable-length signed integer. \see \ref WriteVarUInt */
FORCEINLINE uint32 WriteVarInt(int32 InValue, void* OutData)
{
	const uint32 Value = uint32((InValue >> 31) ^ (InValue << 1));
	return WriteVarUInt(Value, OutData);
}

/** Write a variable-length signed integer. \see \ref WriteVarUInt */
FORCEINLINE uint32 WriteVarInt(int64 InValue, void* OutData)
{
	const uint64 Value = uint64((InValue >> 63) ^ (InValue << 1));
	return WriteVarUInt(Value, OutData);
}

class FArchive;

CORE_API int64 ReadVarIntFromArchive(FArchive& Ar);
CORE_API void WriteVarIntToArchive(FArchive& Ar, int64 Value);
CORE_API void SerializeVarInt(FArchive& Ar, int64& Value);

CORE_API uint64 ReadVarUIntFromArchive(FArchive& Ar);
CORE_API void WriteVarUIntToArchive(FArchive& Ar, uint64 Value);
CORE_API void SerializeVarUInt(FArchive& Ar, uint64& Value);
