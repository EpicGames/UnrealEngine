// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/CustomVersion.h"

// Use TVersionedReader/TVersionedWriter to create an FArchiveProxy derived class using the provided 
// Reader/Writer class that'll automatically serialize the CustomVersions that were used during serialization.

template<class T> 
struct TUnderlyingArchiveContainer
{
	T Ar;

	template <typename... ArgsType>
	explicit TUnderlyingArchiveContainer(ArgsType&&... Args)
		: Ar(Forward<ArgsType>(Args)...)
	{
	}
};

template<class T> 
class TVersionedReader : public TUnderlyingArchiveContainer<T>, public FArchiveProxy
{	
public:
	
	template <typename... ArgsType>
	explicit TVersionedReader(ArgsType&&... Args)
		: TUnderlyingArchiveContainer<T>(Forward<ArgsType>(Args)...)
		, FArchiveProxy(this->Ar)
		
	{		
		FCustomVersionContainer CustomVersions;

		// Serialize the offset to the CustomVersions
		int64 VersionOffset = 0;
		*this << VersionOffset;

		// Preserve where we are so we can return and continue serializing after reading the CustomVersions
		int64 ReturnOffset = this->Tell();
		checkf(ReturnOffset != INDEX_NONE, TEXT("Underlying FArchive must support Seek/Tell to use TVersionnedReader"));

		// Go to CustomVersions, serialize them and set them in the underlying archive
		this->Seek(VersionOffset);
		CustomVersions.Serialize(*this);
		this->SetCustomVersions(CustomVersions);

		// Return to where we seeked from to be able to continue serialization
		this->Seek(ReturnOffset);
	}
};

template<class T>
class TVersionedWriter : public TUnderlyingArchiveContainer<T>, public FArchiveProxy
{	
	int64 VersionOffset = 0;

public:
		
		template <typename... ArgsType>
		explicit TVersionedWriter(ArgsType&&... Args)
			: TUnderlyingArchiveContainer<T>(Forward<ArgsType>(Args)...)
			, FArchiveProxy(this->Ar)
		{
			// Reserve space for version offset and remember where to write it
			VersionOffset = this->Tell();
			checkf(VersionOffset != INDEX_NONE, TEXT("Underlying FArchive must support Seek/Tell to use TVersionnedWriter"));
			*this << VersionOffset;
		}
	
		~TVersionedWriter()
		{
			// Acquire the current set of CustomVersions
			FCustomVersionContainer CustomVersions;
			CustomVersions = this->GetCustomVersions();

			// Capture the offset where it'll be serialized, and then serialize the CustomVersion
			int64 CurrentOffset = this->Tell();
			CustomVersions.Serialize(*this);
			
			// Go back to the initial offset and serialize the CustomVersions offset
			this->Seek(VersionOffset);
			*this << CurrentOffset;
		}
};
