// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AutoRTFM.h"
#include "Misc/Build.h"
#include "Stats/LightweightStats.h"
#include "Stats/StatsSystemTypes.h"

/** Helper class used to generate dynamic stat ids. */
struct FDynamicStats
{
	/**
	* Create a new stat id and registers it with the stats system.
	* This is the only way to create dynamic stat ids at runtime.
	* Can be used only with FScopeCycleCounters.
	*
	* Store the created stat id.
	* Expensive method, avoid calling that method every frame.
	*
	* Example:
	*	FDynamicStats::CreateStatId<STAT_GROUP_TO_FStatGroup( STATGROUP_UObjects )>( FString::Printf(TEXT("MyDynamicStat_%i"),Index) )
	*/
	template< typename TStatGroup >
	static TStatId CreateStatId(const FString& StatNameOrDescription)
	{
#if	STATS
		return CreateStatIdInternal<TStatGroup>(FName(*StatNameOrDescription), EStatDataType::ST_int64, true);
#else
		return TStatId();
#endif // STATS
	}

	template< typename TStatGroup >
	static TStatId CreateStatIdInt64(const FString& StatNameOrDescription, bool bIsAccumulator = false)
	{
#if	STATS
		return CreateStatIdInternal<TStatGroup>(FName(*StatNameOrDescription), EStatDataType::ST_int64, false, !bIsAccumulator);
#else
		return TStatId();
#endif // STATS
	}

	template< typename TStatGroup >
	static TStatId CreateStatIdDouble(const FString& StatNameOrDescription, bool bIsAccumulator = false)
	{
#if	STATS
		return CreateStatIdInternal<TStatGroup>(FName(*StatNameOrDescription), EStatDataType::ST_double, false, !bIsAccumulator);
#else
		return TStatId();
#endif // STATS
	}

	template< typename TStatGroup >
	static TStatId CreateStatId(const FName StatNameOrDescription, bool IsTimer = true)
	{
#if	STATS
		return CreateStatIdInternal<TStatGroup>(StatNameOrDescription, EStatDataType::ST_int64, IsTimer);
#else
		return TStatId();
#endif // STATS
	}

	template< typename TStatGroup >
	static TStatId CreateMemoryStatId(const FString& StatNameOrDescription, FPlatformMemory::EMemoryCounterRegion MemRegion = FPlatformMemory::MCR_Physical)
	{
#if	STATS
		return CreateMemoryStatIdInternal<TStatGroup>(FName(*StatNameOrDescription), MemRegion);
#else
		return TStatId();
#endif // STATS
	}

	template< typename TStatGroup >
	static TStatId CreateMemoryStatId(const FName StatNameOrDescription, FPlatformMemory::EMemoryCounterRegion MemRegion = FPlatformMemory::MCR_Physical)
	{
#if	STATS
		return CreateMemoryStatIdInternal<TStatGroup>(StatNameOrDescription, MemRegion);
#else
		return TStatId();
#endif // STATS
	}

#if	STATS
private: // private since this can only be declared if STATS is defined, due to EStatDataType in signature
	template< typename TStatGroup >
	static TStatId CreateStatIdInternal(const FName StatNameOrDescription, EStatDataType::Type Type, bool IsTimer, bool bClearEveryFrame = true)
	{
		return AutoRTFM::Open([&]
		{
			return CreateStatIdInternalInTheOpen<TStatGroup>(StatNameOrDescription, Type, IsTimer, bClearEveryFrame);
		});
	}

	template< typename TStatGroup >
	AUTORTFM_DISABLE static TStatId CreateStatIdInternalInTheOpen(const FName StatNameOrDescription, EStatDataType::Type Type, bool IsTimer, bool bClearEveryFrame)
	{
		FStartupMessages::Get().AddMetadata(StatNameOrDescription, nullptr,
			TStatGroup::GetGroupName(),
			TStatGroup::GetGroupCategory(),
			TStatGroup::GetDescription(),
			bClearEveryFrame, Type, IsTimer, false);

		TStatId StatID = IStatGroupEnableManager::Get().GetHighPerformanceEnableForStat(StatNameOrDescription,
			TStatGroup::GetGroupName(),
			TStatGroup::GetGroupCategory(),
			TStatGroup::DefaultEnable,
			bClearEveryFrame, Type, nullptr, IsTimer, false);

		return StatID;
	}

	template< typename TStatGroup >
	static TStatId CreateMemoryStatIdInternal(const FName StatNameOrDescription, FPlatformMemory::EMemoryCounterRegion MemRegion = FPlatformMemory::MCR_Physical)
	{
		return AutoRTFM::Open([&]
		{
			return CreateMemoryStatIdInternalInTheOpen<TStatGroup>(StatNameOrDescription, MemRegion);
		});
	}

	template< typename TStatGroup >
	AUTORTFM_DISABLE static TStatId CreateMemoryStatIdInternalInTheOpen(const FName StatNameOrDescription, FPlatformMemory::EMemoryCounterRegion MemRegion = FPlatformMemory::MCR_Physical)
	{
		FStartupMessages::Get().AddMetadata(StatNameOrDescription, *StatNameOrDescription.ToString(),
			TStatGroup::GetGroupName(),
			TStatGroup::GetGroupCategory(),
			TStatGroup::GetDescription(),
			false, EStatDataType::ST_int64, false, false, MemRegion);

		TStatId StatID = IStatGroupEnableManager::Get().GetHighPerformanceEnableForStat(StatNameOrDescription,
			TStatGroup::GetGroupName(),
			TStatGroup::GetGroupCategory(),
			TStatGroup::DefaultEnable,
			false, EStatDataType::ST_int64, *StatNameOrDescription.ToString(), false, false, MemRegion);

		return StatID;
	}
#endif // STATS
};
