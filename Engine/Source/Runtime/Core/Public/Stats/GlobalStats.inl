// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Unique group identifiers. Note these don't have to defined in this header
 * but they do have to be unique. You're better off defining these in your
 * own headers/cpp files
 */
DECLARE_STATS_GROUP(TEXT("AI"), STATGROUP_AI, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Anim"), STATGROUP_Anim, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Async I/O"), STATGROUP_AsyncIO, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Audio"), STATGROUP_Audio, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Beam Particles"), STATGROUP_BeamParticles, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("CPU Stalls"), STATGROUP_CPUStalls, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Canvas"), STATGROUP_Canvas, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Character"), STATGROUP_Character, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Collision"), STATGROUP_Collision, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("CollisionTags"), STATGROUP_CollisionTags, STATCAT_Advanced);
DECLARE_STATS_GROUP_VERBOSE(TEXT("CollisionVerbose"), STATGROUP_CollisionVerbose, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("D3D11RHI"), STATGROUP_D3D11RHI, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("DDC"), STATGROUP_DDC, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Default Stat Group"), STATGROUP_Default, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Engine"), STATGROUP_Engine, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("FPS Chart"), STATGROUP_FPSChart, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("GPU Particles"), STATGROUP_GPUParticles, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Game"), STATGROUP_Game, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("GPU Defrag"), STATGROUP_GPUDEFRAG, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Gnm"), STATGROUP_PS4RHI, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Init Views"), STATGROUP_InitViews, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Landscape"), STATGROUP_Landscape, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Light Rendering"), STATGROUP_LightRendering, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("LoadTime"), STATGROUP_LoadTime, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("LoadTimeClass"), STATGROUP_LoadTimeClass, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("LoadTimeClassCount"), STATGROUP_LoadTimeClassCount, STATCAT_Advanced);
DECLARE_STATS_GROUP_VERBOSE(TEXT("LoadTimeVerbose"), STATGROUP_LoadTimeVerbose, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Media"), STATGROUP_Media, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Memory Allocator"), STATGROUP_MemoryAllocator, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Memory Platform"), STATGROUP_MemoryPlatform, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Memory StaticMesh"), STATGROUP_MemoryStaticMesh, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Memory"), STATGROUP_Memory, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Mesh Particles"), STATGROUP_MeshParticles, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Metal"), STATGROUP_MetalRHI, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("AGX"), STATGROUP_AGXRHI, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Morph"), STATGROUP_MorphTarget, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Navigation"), STATGROUP_Navigation, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Net"), STATGROUP_Net, STATCAT_Advanced);

#if !UE_BUILD_SHIPPING
DECLARE_STATS_GROUP(TEXT("Packet"), STATGROUP_Packet, STATCAT_Advanced);
#endif

DECLARE_STATS_GROUP(TEXT("Object"), STATGROUP_Object, STATCAT_Advanced);
DECLARE_STATS_GROUP_VERBOSE(TEXT("ObjectVerbose"), STATGROUP_ObjectVerbose, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("OpenGL RHI"), STATGROUP_OpenGLRHI, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Pak File"), STATGROUP_PakFile, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Particle Mem"), STATGROUP_ParticleMem, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Particles"), STATGROUP_Particles, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Physics"), STATGROUP_Physics, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Platform"), STATGROUP_Platform, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Profiler"), STATGROUP_Profiler, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Quick"), STATGROUP_Quick, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("RHI"), STATGROUP_RHI, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("RDG"), STATGROUP_RDG, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Render Thread"), STATGROUP_RenderThreadProcessing, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Render Target Pool"), STATGROUP_RenderTargetPool, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Render Scaling"), STATGROUP_RenderScaling, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Scene Memory"), STATGROUP_SceneMemory, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Scene Rendering"), STATGROUP_SceneRendering, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Scene Update"), STATGROUP_SceneUpdate, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Server CPU"), STATGROUP_ServerCPU, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("MapBuildData"), STATGROUP_MapBuildData, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Shader Compiling"), STATGROUP_ShaderCompiling, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Shader Compression"), STATGROUP_Shaders, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Shadow Rendering"), STATGROUP_ShadowRendering, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Stat System"), STATGROUP_StatSystem, STATCAT_Advanced);
DECLARE_STATS_GROUP_SORTBYNAME(TEXT("Streaming Overview"), STATGROUP_StreamingOverview, STATCAT_Advanced);
DECLARE_STATS_GROUP_SORTBYNAME(TEXT("Streaming Details"), STATGROUP_StreamingDetails, STATCAT_Advanced);
DECLARE_STATS_GROUP_SORTBYNAME(TEXT("Streaming"), STATGROUP_Streaming, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Target Platform"), STATGROUP_TargetPlatform, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Text"), STATGROUP_Text, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("ThreadPool Async Tasks"), STATGROUP_ThreadPoolAsyncTasks, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Threading"), STATGROUP_Threading, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Threads"), STATGROUP_Threads, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Tickables"), STATGROUP_Tickables, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Trail Particles"), STATGROUP_TrailParticles, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("UI"), STATGROUP_UI, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("UObjects"), STATGROUP_UObjects, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("User"), STATGROUP_User, STATCAT_Advanced);

DECLARE_CYCLE_STAT_EXTERN(TEXT("FrameTime"), STAT_FrameTime, STATGROUP_Engine, CORE_API);
DECLARE_FNAME_STAT_EXTERN(TEXT("NamedMarker"), STAT_NamedMarker, STATGROUP_StatSystem, CORE_API);
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Seconds Per Cycle"), STAT_SecondsPerCycle, STATGROUP_Engine, CORE_API);