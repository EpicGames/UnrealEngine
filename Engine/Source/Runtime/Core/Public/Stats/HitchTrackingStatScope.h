// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Build.h"
#include "Stats/StatsCommon.h"

#ifndef USE_LIGHTWEIGHT_STATS_FOR_HITCH_DETECTION
#define USE_LIGHTWEIGHT_STATS_FOR_HITCH_DETECTION 1
#endif

#if USE_LIGHTWEIGHT_STATS_FOR_HITCH_DETECTION && USE_HITCH_DETECTION
extern CORE_API TSAN_ATOMIC(bool) GHitchDetected;

namespace UE::Stats
{
	class FHitchTrackingStatScope
	{
		const PROFILER_CHAR* StatString;
	public:
		UE_FORCEINLINE_HINT FHitchTrackingStatScope(const PROFILER_CHAR* InStat)
		{
			StatString = GHitchDetected ? nullptr : InStat;
		}

		inline ~FHitchTrackingStatScope()
		{
			if (GHitchDetected && StatString)
			{
				ReportHitch();
			}
		}

		CORE_API void ReportHitch();
	};
} // namespace UE::Stats

#endif // USE_LIGHTWEIGHT_STATS_FOR_HITCH_DETECTION && USE_HITCH_DETECTION
