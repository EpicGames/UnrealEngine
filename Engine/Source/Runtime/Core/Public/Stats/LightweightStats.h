// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AutoRTFM/ReturnFromOpenMode.h"
#include "CoreTypes.h"
#include "CoreGlobals.h"
#include "Misc/Build.h"
#include "StatsCommon.h"

// UE_USE_LIGHTWEIGHT_STATS is defined in StatsCommon.h
#if UE_USE_LIGHTWEIGHT_STATS

#include "AutoRTFM.h"
#include "Hash/Fnv.h"
#include "Math/Color.h"
#include "Misc/NotNull.h"
#include "Stats/HitchTrackingStatScope.h"
#include "Stats/StatIgnoreList.h"

struct TStatId
{
	// TStatId can be returned from AutoRTFM::Open() via a copy-construction in the closed.
	static constexpr AutoRTFM::EReturnFromOpenMode AutoRTFMReturnFromOpenMode = AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed;

	const PROFILER_CHAR* StatString;

	FORCEINLINE constexpr TStatId()
		: StatString(nullptr)
	{
	}

	FORCEINLINE constexpr TStatId(const PROFILER_CHAR* InString)
		: StatString(InString)
	{
	}

	FORCEINLINE constexpr const PROFILER_CHAR* GetName() const
	{
		return StatString;
	}

	FORCEINLINE constexpr bool IsValidStat() const
	{
		return StatString != nullptr;
	}

	FORCEINLINE constexpr bool operator==(TStatId Other) const
	{
		return StatString == Other.StatString;
	}

	FORCEINLINE constexpr bool operator!=(TStatId Other) const
	{
		return StatString != Other.StatString;
	}
};

namespace UE::Stats::Private
{
	// A TStatId that we've already checked can be emitted (via GetStatIfEnabled).
	struct FCheckedStat
	{
		// Validity is separate from the stat name being not-null
		// as even if the stat shouldn't be emitted as a named event, we may need the stat name for other
		// things such as the FHitchTrackingStatScope.
		// However, it's worth noting the StatString can be null for cases where we do not want to emit a hitch scope
		// which is the case for conditional scopes when the condition is false.
		static constexpr FCheckedStat MakeValid(const PROFILER_CHAR* InStatString)
		{
			return FCheckedStat{ InStatString, true };
		}

		static constexpr FCheckedStat MakeInvalid(const PROFILER_CHAR* InStatString)
		{
			return FCheckedStat{ InStatString, false };
		}

		// Allow implicit conversions to TStatId as FCheckedStat can be assigned to TStatId via GetStatIfEnabled.
		constexpr operator TStatId() const
		{
			// Systems that use TStatId directly test validity through the stat string.
			return bIsValid ? TStatId(StatString) : TStatId{};
		}

		// Returns the stat name, even if this stat isn't valid.
		constexpr const PROFILER_CHAR* GetName() const
		{
			return StatString;
		}

		// Returns true if this stat should be emitted.
		constexpr bool IsValidStat() const
		{
			return bIsValid;
		}

	private:
		constexpr explicit FCheckedStat(const PROFILER_CHAR* InStatString, bool bValidStat)
			: StatString(InStatString)
			, bIsValid(bValidStat)
		{
		}

		const PROFILER_CHAR* StatString;
		bool bIsValid;
	};

	template<typename DerivedScopeCounterType, typename StatIdType = TStatId>
	class TScopeCycleCounterBase
	{
	public:
		FORCEINLINE constexpr explicit TScopeCycleCounterBase(const StatIdType& InStatId)
#if USE_LIGHTWEIGHT_STATS_FOR_HITCH_DETECTION && USE_HITCH_DETECTION
			: StatScope(InStatId.GetName())
#endif
		{
			if (DerivedScopeCounterType::CanEmitStat(InStatId))
			{
				UE_AUTORTFM_OPEN
				{
					DerivedScopeCounterType::BeginNamedEvent(InStatId.GetName());
					bPop = true;
				};
				AutoRTFM::PushOnAbortHandler(this, []() { FPlatformMisc::EndNamedEvent(); });
			}
		}

		FORCEINLINE ~TScopeCycleCounterBase()
		{
			if (bPop)
			{
				AutoRTFM::PopOnAbortHandler(this);
				UE_AUTORTFM_OPEN
				{
					FPlatformMisc::EndNamedEvent();
				};
			}
		}

		FORCEINLINE static bool CanEmitStat(const TStatId& Stat)
		{
			return Stat.IsValidStat() && GCycleStatsShouldEmitNamedEvents > 0;
		}

	private:
#if USE_LIGHTWEIGHT_STATS_FOR_HITCH_DETECTION && USE_HITCH_DETECTION
		UE::Stats::FHitchTrackingStatScope StatScope;
#endif
		bool bPop = false;
	};

	/**
	 * Scope counter for stat names with static storage.
	 */
	class FScopeCycleCounterStatic : TScopeCycleCounterBase<FScopeCycleCounterStatic, FCheckedStat>
	{
		using Base = TScopeCycleCounterBase<FScopeCycleCounterStatic, FCheckedStat>;
	public:
		using Base::CanEmitStat;

		FORCEINLINE constexpr explicit FScopeCycleCounterStatic(const FCheckedStat& InStatId)
			: Base(InStatId)
		{
		}

		FORCEINLINE static constexpr bool CanEmitStat(const FCheckedStat& Stat)
		{
			// If we were given a FCheckedStat it means it's gone through GetIfStatEnabled so
			// GCycleStatsShouldEmitNamedEvents has already been checked.
			return Stat.IsValidStat();
		}

		FORCEINLINE static void BeginNamedEvent(TNotNull<const PROFILER_CHAR* const> StatName)
		{
#if PLATFORM_IMPLEMENTS_BeginNamedEventStatic
			FPlatformMisc::BeginNamedEventStatic(FColor(0), StatName);
#else
			FPlatformMisc::BeginNamedEvent(FColor(0), StatName);
#endif
		}
	};

} // namespace UE::Stats::Private

// ----------------------------------------------------------------
// FScopeCycleCounter
// ----------------------------------------------------------------

// Scope counter for stat names with dynamic storage.
class FScopeCycleCounter : private UE::Stats::Private::TScopeCycleCounterBase<FScopeCycleCounter>
{
	using Base = UE::Stats::Private::TScopeCycleCounterBase<FScopeCycleCounter>;

public:
	using Base::CanEmitStat;

	// NOTE: this signature must match the other FScopeCycleCounter implementations
	FORCEINLINE FScopeCycleCounter(TStatId InStatId, EStatFlags, bool bAlways = false)
		: Base(InStatId)
	{
	}

	// NOTE: this signature must match the other FScopeCycleCounter implementations
	FORCEINLINE FScopeCycleCounter(TStatId InStatId, bool bAlways = false)
		: FScopeCycleCounter(InStatId, EStatFlags::None, bAlways)
	{
	}

	FORCEINLINE static void BeginNamedEvent(TNotNull<const PROFILER_CHAR* const> StatName)
	{
		FPlatformMisc::BeginNamedEvent(FColor(0), StatName);
	}
};

// ----------------------------------------------------------------
// Internal Helpers
// ----------------------------------------------------------------
namespace UE::Stats::Private
{
	// ----------------------------------------------------------------
	// Stat/Group Getter Helpers
	// ----------------------------------------------------------------
	
	// Fallback group data for groups that don't have a custom group struct defined.
	struct FDefaultGroupData
	{
		static constexpr const PROFILER_CHAR* GetName()
		{
			return nullptr;
		}
		static constexpr uint32 GetNameHash()
		{
			return 0;
		}
		// Note: matches spelling of implementation when STATS is 1.
		static constexpr bool IsCompileTimeEnable()
		{
			return true;
		}
	};

	// Fallback stat data for StatId's that don't have a custom stat struct defined.
	struct FDefaultStatData
	{
		using TGroup = FDefaultGroupData;

		static constexpr const PROFILER_CHAR* GetName()
		{
			return nullptr;
		}
		static constexpr EStatFlags GetFlags()
		{
			return EStatFlags::None;
		}
	};

	template<typename StatDataType, bool AllowDefault, typename = void>
	struct TStatDataHelper
	{
		// This error means a DECLARE_* macro is missing for this stat for this compile configuration.
		// The Declaration macros can be found in Stats.h.
		static_assert(AllowDefault, "Default StatId not permitted. Please provide a declaration for this stat via one of the DECLARE_ Stat macros.");
		using Type = FDefaultStatData;
	};
	// If StatDataType is a complete type such that we can call sizeof on it, this specialization is used
	template<typename StatDataType, bool AllowDefault>
	struct TStatDataHelper<StatDataType, AllowDefault, std::void_t<decltype(sizeof(StatDataType))>>
	{
		using Type = StatDataType;
	};

	// Helper defined as either the custom stat data struct if it's defined, or the default (FDefaultStatData).
	template<typename StatDataType, bool AllowDefault>
	using TStatDataType = typename TStatDataHelper<StatDataType, AllowDefault>::Type;


	template<typename AllowDefaultStatToken, typename = void>
	struct TAllowDefaultStatHelper
	{
		static constexpr bool Value = false;
	};
	template<typename AllowDefaultStatToken>
	struct TAllowDefaultStatHelper<AllowDefaultStatToken, std::void_t<decltype(sizeof(AllowDefaultStatToken))>>
	{
		// A token has been defined for this stat, so it's allowed.
		static constexpr bool Value = true;
	};

	// Determines if falling back to FDefaultStatData is valid for a given stat.
	// To be allowed, INTERNAL_ALLOW_DEFAULT_STAT must be defined for the stat.
	template<typename AllowDefaultStatToken>
	constexpr bool AllowDefaultForStat = TAllowDefaultStatHelper<AllowDefaultStatToken>::Value;

	template<typename GroupDataType, typename = void>
	struct TGroupDataHelper
	{
		using Type = FDefaultGroupData;
	};
	template<typename GroupDataType>
	struct TGroupDataHelper<GroupDataType, std::void_t<decltype(sizeof(GroupDataType))>>
	{
		using Type = GroupDataType;
	};

	// Helper defined as either the custom group data struct if it's defined, or the default (FDefaultGroupData).
	template<typename GroupDataType>
	using TGroupDataType = typename TGroupDataHelper<GroupDataType>::Type;

	// Tests if this stat can be emitted
	template<typename StatDataType = FDefaultStatData, typename GroupDataType = typename StatDataType::TGroup>
	inline bool IsStatEnabled(uint32 StatNameHash, uint32 GroupNameHash = 0)
	{
		if constexpr (!GroupDataType::IsCompileTimeEnable())
		{
			return false;
		}
		else
		{
			if constexpr (EnumHasAnyFlags(StatDataType::GetFlags(), EStatFlags::Verbose))
			{
				if (!GShouldEmitVerboseNamedEvents)
				{
					return false;
				}
			}

#if UE_STATS_ALLOW_PER_THREAD_IGNORELIST
			const uint32 GroupHash = GroupNameHash != 0 ? GroupNameHash : GroupDataType::GetNameHash();
			if (UE::Stats::IsStatOrGroupIgnoredOnCurrentThread(StatNameHash, GroupHash))
			{
				return false;
			}
#endif // UE_STATS_ALLOW_PER_THREAD_IGNORELIST
			return true;
		}
	}

	/**
	 * Returns a valid TStatId if we can emit it.
	 * 
	 * StatNameHash and GroupNameHash must be calculated at compile time to avoid a very large overhead.
	 * GroupNameHash can be 0 if the stat has no group or the group is default.
	 * Both of these values will be 0 if UE_STATS_ALLOW_PER_THREAD_IGNORELIST is 0.
	 */
	template<typename StatDataType = FDefaultStatData, typename GroupDataType = typename StatDataType::TGroup>
	FORCEINLINE FCheckedStat GetStatIfEnabled(TNotNull<const PROFILER_CHAR*> StatName, uint32 StatNameHash, uint32 GroupNameHash = 0)
	{
		// We're testing if named events are enabled here as it's cheaper than some of the checks in IsStatEnabled.
		if (GCycleStatsShouldEmitNamedEvents > 0 &&
			IsStatEnabled<StatDataType, GroupDataType>(StatNameHash, GroupNameHash))
		{
			return FCheckedStat::MakeValid(StatName);
		}
		return FCheckedStat::MakeInvalid(StatName);
	}

	/* Returns a valid TStatId if the condition is true and we can emit it. */
	template<typename StatDataType = FDefaultStatData, typename GroupDataType = typename StatDataType::TGroup>
	FORCEINLINE FCheckedStat GetStatIfEnabled(TNotNull<const PROFILER_CHAR*> StatName, bool bCondition, uint32 StatNameHash, uint32 GroupNameHash = 0)
	{
		if (bCondition)
		{
			return GetStatIfEnabled<StatDataType, GroupDataType>(StatName, StatNameHash, GroupNameHash);
		}
		// Preserve existing behavior of not emitting hitch scopes if the condition is false.
		return FCheckedStat::MakeInvalid(nullptr);
	}
} // namespace UE::Stats::Private

// ----------------------------------------------------------------
// Helper Macros
// ----------------------------------------------------------------

#define UE_INTERNAL_GET_STATGROUP_TYPE(GroupId)\
	UE::Stats::Private::TGroupDataType<struct FStatGroup_##GroupId>

// Specifies that a custom FStat struct doesn't need to be defined for this stat and we can fallback to the default.
// This is for internal use only currently.
#define UE_INTERNAL_ALLOW_DEFAULT_STAT(Stat)\
	struct FAllowDefaultStat_##Stat{};

// Returns either the custom stat data struct for a stat, or FDefaultStatData if one doesn't exist.
#define UE_INTERNAL_GET_STATDATA(Stat)\
	UE::Stats::Private::TStatDataType<struct FStat_##Stat, UE::Stats::Private::AllowDefaultForStat<struct FAllowDefaultStat_##Stat>>

// Returns either the custom stat data struct for a stat, or FDefaultStatData if one doesn't exist.
// This version will not emit a compilation error if a default is not allowed to handled special cases.
#define UE_INTERNAL_GET_STATDATA_ALLOW_DEFAULT(Stat)\
	UE::Stats::Private::TStatDataType<struct FStat_##Stat, true>

// Helper to evaluate if a stat is marked as verbose.
#define UE_IS_STAT_VERBOSE(Stat)\
	EnumHasAnyFlags(UE_INTERNAL_GET_STATDATA(Stat)::GetFlags(), EStatFlags::Verbose)

#define UE_IS_STAT_VERBOSE_ALLOW_DEFAULT(Stat)\
	EnumHasAnyFlags(UE_INTERNAL_GET_STATDATA_ALLOW_DEFAULT(Stat)::GetFlags(), EStatFlags::Verbose)

// Getters
#define UE_INTERNAL_GET_STAT_IF_ENABLED(Stat)\
	UE::Stats::Private::GetStatIfEnabled<UE_INTERNAL_GET_STATDATA(Stat)>(ANSI_TO_PROFILING(#Stat), UE_STATS_HASH_NAME(Stat))

#define UE_INTERNAL_GET_STAT_IF_ENABLED_COND(Stat, Cond)\
	UE::Stats::Private::GetStatIfEnabled<UE_INTERNAL_GET_STATDATA(Stat)>(ANSI_TO_PROFILING(#Stat), (Cond), UE_STATS_HASH_NAME(Stat))

#define UE_INTERNAL_GET_QUICK_STAT_IF_ENABLED(Stat)\
	UE::Stats::Private::GetStatIfEnabled<UE_INTERNAL_GET_STATDATA_ALLOW_DEFAULT(Stat)>(ANSI_TO_PROFILING(#Stat), UE_STATS_HASH_NAME(Stat))

#define UE_INTERNAL_GET_QUICK_STAT_WITH_GROUP_IF_ENABLED(Stat, Group)\
	UE::Stats::Private::GetStatIfEnabled<UE_INTERNAL_GET_STATDATA_ALLOW_DEFAULT(Stat), UE_INTERNAL_GET_STATGROUP_TYPE(Group)>(ANSI_TO_PROFILING(#Stat), UE_STATS_HASH_NAME(Stat), UE_STATS_HASH_NAME(Group))

#elif !STATS // UE_USE_LIGHTWEIGHT_STATS

struct TStatId
{
	// TStatId can be returned from AutoRTFM::Open() via a copy-construction in the closed.
	static constexpr AutoRTFM::EReturnFromOpenMode AutoRTFMReturnFromOpenMode = AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed;

	inline constexpr bool IsValidStat() const
	{
		return false;
	}
};

class FScopeCycleCounter
{
public:
	inline FScopeCycleCounter(TStatId, EStatFlags, bool bAlways = false)
	{
	}
	inline FScopeCycleCounter(TStatId, bool bAlways = false)
	{
	}
};

#endif // UE_USE_LIGHTWEIGHT_STATS
