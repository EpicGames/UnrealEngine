// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Hash/Fnv.h"
#include "Stats/StatsCommon.h"

#ifndef UE_STATS_ALLOW_PER_THREAD_IGNORELIST
// TODO: support in regular stats as well.
#define UE_STATS_ALLOW_PER_THREAD_IGNORELIST UE_USE_LIGHTWEIGHT_STATS
#endif

/**
 * This is an internal API that allows individual stats or groups to be ignored on certain threads such that they will not be emitted to profilers.
 * 
 * The primary use case for this is to silence super verbose stats when they're only a problem on certain threads but still desired on others.
 * Marking stats as Verbose or removing them entirely is still preferred before utilizing this system as there is a slight cost that 
 * scales with the number of ignored stats (only while named events are enabled).
 * 
 * The per-thread ignore lists can be configured per platform as they are read from the *Engine.ini file. See InitializeIgnoreList for details.
 */
#if UE_STATS_ALLOW_PER_THREAD_IGNORELIST
namespace UE::Stats
{
	/**
	 * Loads the ignored stats/groups from the engine config and initializes the ignore list.
	 * IsStatIgnored/IsStatOrGroupIgnored will always return false until this initialization has occured.
	 * 
	 * Example config setup:
	 * [Stats.PerThreadIgnoreList]
	 * IgnoreListEnabled=true ; enable the system
	 * IgnoredStats=STAT_MyStat:GameThread|RenderingThread ; ignore STAT_MyStat on the GameThread and RenderingThread.
	 * +IgnoredStats=STAT_MyOtherStat:RhiThread
	 * IgnoredGroups=STATGROUP_Object:AsyncLoadingThread ; Ignore this entire group on the AsyncLoadingThread.
	 * +IgnoredGroups=STATGROUP_ObjectVerbose:AsyncLoadingThread
	 * 
	 * See NamedThreadMap in StatIgnoreList.cpp for the full list of valid threads.
	 */
	void InitializeIgnoreList();

	/**
	 * Returns true if the stat is ignored on this thread.
	 * 
	 * NOTE: This API is only meant to be called internally by the various STAT macros.
	 * 
	 * The stat is identified by the FNV1a hash of its name rather than the raw string for sake of performance.
	 * You can use UE::HashStringFNV1a32 to generate this hash, or use UE_STATS_HASH_NAME to calculate it at compile time.
	 * 
	 * @param StatNameHash: The 32-bit FNV1a hash of the stat name.
	 * @param GroupNameHash: The 32-bit FNV1a hash of the group name this stat is part of, or 0 to skip checking the group.
	 */
	bool CORE_API IsStatOrGroupIgnoredOnCurrentThread(uint32 StatNameHash, uint32 GroupNameHash = 0);

} // namespace UE::Stats
#endif // UE_STATS_ALLOW_PER_THREAD_IGNORELIST

#if UE_STATS_ALLOW_PER_THREAD_IGNORELIST
	// Hashes the name at compile time.
	#define UE_STATS_HASH_NAME(Name)\
		UE::HashStringFNV1a<uint32>(TEXT(#Name))
#else
	#define UE_STATS_HASH_NAME(Name) 0
#endif
