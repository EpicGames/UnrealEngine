// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "StatsCommon.h"
#include "Stats/DynamicStats.h"
#include "Stats/HitchTrackingStatScope.h"
#include "Stats/LightweightStats.h"
#include "Stats/StatsSystemTypes.h"

// used by the profiler
enum EStatType
{
	STATTYPE_CycleCounter,
	STATTYPE_AccumulatorFLOAT,
	STATTYPE_AccumulatorDWORD,
	STATTYPE_CounterFLOAT,
	STATTYPE_CounterDWORD,
	STATTYPE_MemoryCounter,
	STATTYPE_Error
};

#if STATS
	#define STAT(x) x
#else
	#define STAT(x)
#endif

#if STATS
// ----------------------------------------------------------------------------
// Stats
// ----------------------------------------------------------------------------
// These macros emit events to the stat system as well as to profilers.
// See official docs for more information on the Stats System.
// ----------------------------------------------------------------------------

#define DECLARE_STAT_GROUP(Description, StatName, StatCategory, InDefaultEnable, InCompileTimeEnable, InSortByName) \
struct FStatGroup_##StatName\
{ \
	enum \
	{ \
		DefaultEnable = InDefaultEnable, \
		CompileTimeEnable = InCompileTimeEnable, \
		SortByName = InSortByName \
	}; \
	static UE_FORCEINLINE_HINT const char* GetGroupName() \
	{ \
		return #StatName; \
	} \
	static UE_FORCEINLINE_HINT const char* GetGroupCategory() \
	{ \
		return #StatCategory; \
	} \
	static UE_FORCEINLINE_HINT const TCHAR* GetDescription() \
	{ \
		return Description; \
	} \
	static UE_FORCEINLINE_HINT bool IsDefaultEnabled() \
	{ \
		return (bool)DefaultEnable; \
	} \
	static UE_FORCEINLINE_HINT bool IsCompileTimeEnable() \
	{ \
		return (bool)CompileTimeEnable; \
	} \
	static UE_FORCEINLINE_HINT bool GetSortByName() \
	{ \
		return (bool)SortByName; \
	} \
};

#define DECLARE_STAT(Description, StatName, GroupName, StatType, StatFlags, MemoryRegion) \
struct FStat_##StatName\
{ \
	typedef FStatGroup_##GroupName TGroup; \
	static UE_FORCEINLINE_HINT const char* GetStatName() \
	{ \
		return #StatName; \
	} \
	static UE_FORCEINLINE_HINT const TCHAR* GetDescription() \
	{ \
		return Description; \
	} \
	static UE_FORCEINLINE_HINT EStatDataType::Type GetStatType() \
	{ \
		return StatType; \
	} \
	static UE_FORCEINLINE_HINT bool IsClearEveryFrame() \
	{ \
		return EnumHasAnyFlags(GetFlags(), EStatFlags::ClearEveryFrame); \
	} \
	static UE_FORCEINLINE_HINT bool IsCycleStat() \
	{ \
		return EnumHasAnyFlags(GetFlags(), EStatFlags::CycleStat); \
	} \
	static UE_FORCEINLINE_HINT EStatFlags GetFlags() \
	{ \
		return StatFlags; \
	} \
	static UE_FORCEINLINE_HINT FPlatformMemory::EMemoryCounterRegion GetMemoryRegion() \
	{ \
		return MemoryRegion; \
	} \
};

#define GET_STATID(Stat) (StatPtr_##Stat.GetStatId())
#define GET_STATFNAME(Stat) (StatPtr_##Stat.GetStatFName())
#define GET_STATDESCRIPTION(Stat) (FStat_##Stat::GetDescription())
#define GET_STATISEVERYFRAME(Stat) (FStat_##Stat::IsClearEveryFrame())
#define GET_STATFLAGS(Stat) (FStat_##Stat::GetFlags())

#define STAT_GROUP_TO_FStatGroup(Group) FStatGroup_##Group

/*-----------------------------------------------------------------------------
	Local
-----------------------------------------------------------------------------*/

#define DEFINE_STAT(Stat) \
	struct FThreadSafeStaticStat<FStat_##Stat> StatPtr_##Stat;

#define RETURN_QUICK_DECLARE_CYCLE_STAT(StatId,GroupId) \
	DECLARE_STAT(TEXT(#StatId),StatId,GroupId,EStatDataType::ST_int64, EStatFlags::ClearEveryFrame | EStatFlags::CycleStat, FPlatformMemory::MCR_Invalid); \
	static DEFINE_STAT(StatId) \
	return GET_STATID(StatId);

#define QUICK_USE_CYCLE_STAT(StatId,GroupId) [](){ RETURN_QUICK_DECLARE_CYCLE_STAT(StatId, GroupId); }()

#define DECLARE_CYCLE_STAT(CounterName,StatId,GroupId) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_int64, EStatFlags::ClearEveryFrame | EStatFlags::CycleStat, FPlatformMemory::MCR_Invalid); \
	static DEFINE_STAT(StatId)
#define DECLARE_CYCLE_STAT_WITH_FLAGS(CounterName,StatId,GroupId,StatFlags) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_int64, (StatFlags) | EStatFlags::ClearEveryFrame | EStatFlags::CycleStat, FPlatformMemory::MCR_Invalid); \
	static DEFINE_STAT(StatId)
#define DECLARE_FLOAT_COUNTER_STAT(CounterName,StatId,GroupId) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_double,EStatFlags::ClearEveryFrame, FPlatformMemory::MCR_Invalid); \
	static DEFINE_STAT(StatId)
#define DECLARE_DWORD_COUNTER_STAT(CounterName,StatId,GroupId) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_int64,EStatFlags::ClearEveryFrame, FPlatformMemory::MCR_Invalid); \
	static DEFINE_STAT(StatId)
#define DECLARE_FLOAT_ACCUMULATOR_STAT(CounterName,StatId,GroupId) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_double, EStatFlags::None, FPlatformMemory::MCR_Invalid); \
	static DEFINE_STAT(StatId)
#define DECLARE_DWORD_ACCUMULATOR_STAT(CounterName,StatId,GroupId) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_int64, EStatFlags::None, FPlatformMemory::MCR_Invalid); \
	static DEFINE_STAT(StatId)

/** FName stat that allows sending a string based data. */
#define DECLARE_FNAME_STAT(CounterName,StatId,GroupId) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_FName, EStatFlags::None, FPlatformMemory::MCR_Invalid); \
	static DEFINE_STAT(StatId)

/** This is a fake stat, mostly used to implement memory message or other custom stats that don't easily fit into the system. */
#define DECLARE_PTR_STAT(CounterName,StatId,GroupId)\
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_Ptr, EStatFlags::None, FPlatformMemory::MCR_Invalid); \
	static DEFINE_STAT(StatId)

#define DECLARE_MEMORY_STAT(CounterName,StatId,GroupId) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_int64, EStatFlags::None, FPlatformMemory::MCR_Physical); \
	static DEFINE_STAT(StatId)

#define DECLARE_MEMORY_STAT_POOL(CounterName,StatId,GroupId,Pool) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_int64, EStatFlags::None, Pool); \
	static DEFINE_STAT(StatId)

/*-----------------------------------------------------------------------------
	Extern
-----------------------------------------------------------------------------*/

#define DECLARE_CYCLE_STAT_EXTERN(CounterName,StatId,GroupId, APIX) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_int64, EStatFlags::ClearEveryFrame | EStatFlags::CycleStat, FPlatformMemory::MCR_Invalid); \
	extern APIX DEFINE_STAT(StatId);
#define DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(CounterName,StatId,GroupId,StatFlags, APIX) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_int64, (StatFlags) | EStatFlags::ClearEveryFrame | EStatFlags::CycleStat, FPlatformMemory::MCR_Invalid); \
	extern APIX DEFINE_STAT(StatId);
#define DECLARE_FLOAT_COUNTER_STAT_EXTERN(CounterName,StatId,GroupId, API) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_double, EStatFlags::ClearEveryFrame, FPlatformMemory::MCR_Invalid); \
	extern API DEFINE_STAT(StatId);
#define DECLARE_DWORD_COUNTER_STAT_EXTERN(CounterName,StatId,GroupId, API) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_int64, EStatFlags::ClearEveryFrame, FPlatformMemory::MCR_Invalid); \
	extern API DEFINE_STAT(StatId);
#define DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(CounterName,StatId,GroupId, API) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_double, EStatFlags::None, FPlatformMemory::MCR_Invalid); \
	extern API DEFINE_STAT(StatId);
#define DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(CounterName,StatId,GroupId, API) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_int64, EStatFlags::None, FPlatformMemory::MCR_Invalid); \
	extern API DEFINE_STAT(StatId);

/** FName stat that allows sending a string based data. */
#define DECLARE_FNAME_STAT_EXTERN(CounterName,StatId,GroupId, API) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_FName, EStatFlags::None, FPlatformMemory::MCR_Invalid); \
	extern API DEFINE_STAT(StatId);

/** This is a fake stat, mostly used to implement memory message or other custom stats that don't easily fit into the system. */
#define DECLARE_PTR_STAT_EXTERN(CounterName,StatId,GroupId, API) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_Ptr, EStatFlags::None, FPlatformMemory::MCR_Invalid); \
	extern API DEFINE_STAT(StatId);

#define DECLARE_MEMORY_STAT_EXTERN(CounterName,StatId,GroupId, API) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_int64, EStatFlags::None, FPlatformMemory::MCR_Physical); \
	extern API DEFINE_STAT(StatId);

#define DECLARE_MEMORY_STAT_POOL_EXTERN(CounterName,StatId,GroupId,Pool, API) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_int64, EStatFlags::None, Pool); \
	extern API DEFINE_STAT(StatId);

/** Macro for declaring group factory instances */
#define DECLARE_STATS_GROUP(GroupDesc, GroupId, GroupCat) \
	DECLARE_STAT_GROUP(GroupDesc, GroupId, GroupCat, true, true, false);

#define DECLARE_STATS_GROUP_SORTBYNAME(GroupDesc, GroupId, GroupCat) \
	DECLARE_STAT_GROUP(GroupDesc, GroupId, GroupCat, true, true, true);

#define DECLARE_STATS_GROUP_VERBOSE(GroupDesc, GroupId, GroupCat) \
	DECLARE_STAT_GROUP(GroupDesc, GroupId, GroupCat, false, true, false);

#define DECLARE_STATS_GROUP_MAYBE_COMPILED_OUT(GroupDesc, GroupId, GroupCat, CompileIn) \
	DECLARE_STAT_GROUP(GroupDesc, GroupId, GroupCat, false, CompileIn, false);

#define DECLARE_SCOPE_CYCLE_COUNTER(CounterName,Stat,GroupId) \
	DECLARE_STAT(CounterName,Stat,GroupId,EStatDataType::ST_int64, EStatFlags::ClearEveryFrame | EStatFlags::CycleStat, FPlatformMemory::MCR_Invalid); \
	static DEFINE_STAT(Stat) \
	FScopeCycleCounter CycleCount_##Stat(GET_STATID(Stat), GET_STATFLAGS(Stat));

#define QUICK_SCOPE_CYCLE_COUNTER(Stat) \
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT(#Stat),Stat,STATGROUP_Quick)

#define SCOPE_CYCLE_COUNTER(Stat) \
	FScopeCycleCounter CycleCount_##Stat(GET_STATID(Stat), GET_STATFLAGS(Stat));

#define SCOPE_CYCLE_COUNTER_STATID(StatId) \
	FScopeCycleCounter UE_JOIN(CycleCount_FromStatId_, __LINE__)(StatId);

#define CONDITIONAL_SCOPE_CYCLE_COUNTER(Stat,bCondition) \
	FScopeCycleCounter CycleCount_##Stat(bCondition ? GET_STATID(Stat) : TStatId(), GET_STATFLAGS(Stat));

#define SCOPE_SECONDS_ACCUMULATOR(Stat) \
	FSimpleScopeSecondsStat SecondsAccum_##Stat(GET_STATID(Stat));

#define SCOPE_MS_ACCUMULATOR(Stat) \
	FSimpleScopeSecondsStat SecondsAccum_##Stat(GET_STATID(Stat), 1000.0);

#define SET_CYCLE_COUNTER(Stat,Cycles) \
{\
	if (FThreadStats::IsCollectingData() || !GET_STATISEVERYFRAME(Stat)) \
		FThreadStats::AddMessage(GET_STATFNAME(Stat), EStatOperation::Set, int64(Cycles), true);\
}

#define INC_DWORD_STAT(Stat) \
{\
	if (FThreadStats::IsCollectingData() || !GET_STATISEVERYFRAME(Stat)) \
	{ \
		const FName StatName = GET_STATFNAME(Stat); \
		FThreadStats::AddMessage(StatName, EStatOperation::Add, int64(1));\
	} \
}
#define INC_FLOAT_STAT_BY(Stat, Amount) \
{\
	double AddAmount = double(Amount); \
	if (AddAmount != 0.0) \
	{ \
		if (FThreadStats::IsCollectingData() || !GET_STATISEVERYFRAME(Stat)) \
		{ \
			const FName StatName = GET_STATFNAME(Stat); \
			FThreadStats::AddMessage(StatName, EStatOperation::Add, AddAmount); \
		} \
	} \
}
#define INC_DWORD_STAT_BY(Stat, Amount) \
{\
	int64 AddAmount = int64(Amount); \
	if (AddAmount != 0) \
	{ \
		if (FThreadStats::IsCollectingData() || !GET_STATISEVERYFRAME(Stat)) \
		{ \
			const FName StatName = GET_STATFNAME(Stat); \
			FThreadStats::AddMessage(StatName, EStatOperation::Add, AddAmount); \
		} \
	} \
}
#define INC_MEMORY_STAT_BY(Stat, Amount) \
{\
	int64 AddAmount = int64(Amount); \
	if (AddAmount != 0) \
	{ \
		if (FThreadStats::IsCollectingData() || !GET_STATISEVERYFRAME(Stat)) \
		{ \
			const FName StatName = GET_STATFNAME(Stat); \
			FThreadStats::AddMessage(StatName, EStatOperation::Add, AddAmount); \
		} \
	} \
}
#define DEC_DWORD_STAT(Stat) \
{\
	if (FThreadStats::IsCollectingData() || !GET_STATISEVERYFRAME(Stat)) \
	{ \
		const FName StatName = GET_STATFNAME(Stat); \
		FThreadStats::AddMessage(StatName, EStatOperation::Subtract, int64(1));\
	} \
}
#define DEC_FLOAT_STAT_BY(Stat,Amount) \
{\
	double SubtractAmount = double(Amount); \
	if (SubtractAmount != 0.0) \
	{ \
		if (FThreadStats::IsCollectingData() || !GET_STATISEVERYFRAME(Stat)) \
		{ \
			const FName StatName = GET_STATFNAME(Stat); \
			FThreadStats::AddMessage(StatName, EStatOperation::Subtract, SubtractAmount); \
		} \
	} \
}
#define DEC_DWORD_STAT_BY(Stat,Amount) \
{\
	int64 SubtractAmount = int64(Amount); \
	if (SubtractAmount != 0) \
	{ \
		if (FThreadStats::IsCollectingData() || !GET_STATISEVERYFRAME(Stat)) \
		{ \
			const FName StatName = GET_STATFNAME(Stat); \
			FThreadStats::AddMessage(StatName, EStatOperation::Subtract, SubtractAmount); \
		} \
	} \
}
#define DEC_MEMORY_STAT_BY(Stat,Amount) \
{\
	int64 SubtractAmount = int64(Amount); \
	if (SubtractAmount != 0) \
	{ \
		if (FThreadStats::IsCollectingData() || !GET_STATISEVERYFRAME(Stat)) \
		{ \
			const FName StatName = GET_STATFNAME(Stat); \
			FThreadStats::AddMessage(StatName, EStatOperation::Subtract, SubtractAmount); \
		} \
	} \
}
#define SET_MEMORY_STAT(Stat,Value) \
{\
	if (FThreadStats::IsCollectingData() || !GET_STATISEVERYFRAME(Stat)) \
	{ \
		const FName StatName = GET_STATFNAME(Stat); \
		int64 SetValue = int64(Value); \
		FThreadStats::AddMessage(StatName, EStatOperation::Set, SetValue); \
	} \
}
#define SET_DWORD_STAT(Stat,Value) \
{\
	if (FThreadStats::IsCollectingData() || !GET_STATISEVERYFRAME(Stat)) \
	{ \
		const FName StatName = GET_STATFNAME(Stat); \
		int64 SetValue = int64(Value); \
		FThreadStats::AddMessage(StatName, EStatOperation::Set, SetValue); \
	} \
}
#define SET_FLOAT_STAT(Stat,Value) \
{\
	if (FThreadStats::IsCollectingData() || !GET_STATISEVERYFRAME(Stat)) \
	{ \
		const FName StatName = GET_STATFNAME(Stat); \
		double SetValue = double(Value); \
		FThreadStats::AddMessage(StatName, EStatOperation::Set, SetValue); \
	} \
}

#define STAT_ADD_CUSTOMMESSAGE_NAME(Stat,Value) \
{\
	FThreadStats::AddMessage(GET_STATFNAME(Stat), EStatOperation::SpecialMessageMarker, FName(Value));\
}
#define STAT_ADD_CUSTOMMESSAGE_PTR(Stat,Value) \
{\
	FThreadStats::AddMessage(GET_STATFNAME(Stat), EStatOperation::SpecialMessageMarker, uint64(Value));\
}

#define SET_CYCLE_COUNTER_FName(Stat,Cycles) \
{\
	FThreadStats::AddMessage(Stat, EStatOperation::Set, int64(Cycles), true);\
}

#define INC_DWORD_STAT_FName(Stat) \
{\
	FThreadStats::AddMessage(Stat, EStatOperation::Add, int64(1));\
}
#define INC_FLOAT_STAT_BY_FName(Stat, Amount) \
{\
	double AddAmount = double(Amount); \
	if (AddAmount != 0.0) \
	{ \
		FThreadStats::AddMessage(Stat, EStatOperation::Add, AddAmount); \
	} \
}
#define INC_DWORD_STAT_BY_FName(Stat, Amount) \
{\
	int64 AddAmount = int64(Amount); \
	if (AddAmount != 0) \
	{ \
		FThreadStats::AddMessage(Stat, EStatOperation::Add, AddAmount); \
	} \
}
#define INC_DWORD_STAT_FNAME_BY(Stat, Amount) INC_DWORD_STAT_BY_FName(Stat, Amount)
#define INC_MEMORY_STAT_BY_FName(Stat, Amount) \
{\
	int64 AddAmount = int64(Amount); \
	if (AddAmount != 0) \
	{ \
		FThreadStats::AddMessage(Stat, EStatOperation::Add, AddAmount); \
	} \
}
#define DEC_DWORD_STAT_FName(Stat) \
{\
	FThreadStats::AddMessage(Stat, EStatOperation::Subtract, int64(1));\
}
#define DEC_FLOAT_STAT_BY_FName(Stat,Amount) \
{\
	double SubtractAmount = double(Amount); \
	if (SubtractAmount != 0.0) \
	{ \
		FThreadStats::AddMessage(Stat, EStatOperation::Subtract, SubtractAmount); \
	} \
}
#define DEC_DWORD_STAT_BY_FName(Stat,Amount) \
{\
	int64 SubtractAmount = int64(Amount); \
	if (SubtractAmount != 0) \
	{ \
		FThreadStats::AddMessage(Stat, EStatOperation::Subtract, SubtractAmount); \
	} \
}
#define DEC_DWORD_STAT_FNAME_BY(Stat,Amount) DEC_DWORD_STAT_BY_FName(Stat,Amount)
#define DEC_MEMORY_STAT_BY_FName(Stat,Amount) \
{\
	int64 SubtractAmount = int64(Amount); \
	if (SubtractAmount != 0) \
	{ \
		FThreadStats::AddMessage(Stat, EStatOperation::Subtract, SubtractAmount); \
	} \
}
#define SET_MEMORY_STAT_FName(Stat,Value) \
{\
	int64 SetValue = int64(Value); \
	FThreadStats::AddMessage(Stat, EStatOperation::Set, SetValue); \
}
#define SET_DWORD_STAT_FName(Stat,Value) \
{\
	int64 SetValue = int64(Value); \
	FThreadStats::AddMessage(Stat, EStatOperation::Set, SetValue); \
}
#define SET_FLOAT_STAT_FName(Stat,Value) \
{\
	double SetValue = double(Value); \
	FThreadStats::AddMessage(Stat, EStatOperation::Set, SetValue); \
}

#elif UE_USE_LIGHTWEIGHT_STATS // STATS
// STATS == 0

// ----------------------------------------------------------------------------
// Lightweight Stats
// ----------------------------------------------------------------------------
// This defines a simpler and lower overhead version of the stat macros.
// These will not emit any data to the runtime stat system and only emit
// profiling namedevents if enabled.
// ----------------------------------------------------------------------------

// ----------------------------------------------------------------
// Group Declarations
// ----------------------------------------------------------------

#define DECLARE_STAT_GROUP(GroupId, CompiledIn) \
	struct FStatGroup_##GroupId\
	{\
		static constexpr const PROFILER_CHAR* GetName()\
		{\
			return ANSI_TO_PROFILING(#GroupId);\
		}\
		static constexpr uint32 GetNameHash()\
		{\
			return UE_STATS_HASH_NAME(GroupId);\
		}\
		static constexpr bool IsCompileTimeEnable()\
		{\
			return CompiledIn;\
		}\
	};

#define DECLARE_STATS_GROUP(GroupDesc, GroupId, GroupCat) \
	DECLARE_STAT_GROUP(GroupId, true);

#define DECLARE_STATS_GROUP_SORTBYNAME(GroupDesc, GroupId, GroupCat) \
	DECLARE_STAT_GROUP(GroupId, true);

#define DECLARE_STATS_GROUP_VERBOSE(GroupDesc, GroupId, GroupCat) \
	DECLARE_STAT_GROUP(GroupId, true);

#define DECLARE_STATS_GROUP_MAYBE_COMPILED_OUT(GroupDesc, GroupId, GroupCat, CompileIn) \
	DECLARE_STAT_GROUP(GroupId, static_cast<bool>(CompileIn));

// ----------------------------------------------------------------
// Stat Declarations
// ----------------------------------------------------------------

#define DECLARE_STAT(StatName, GroupName, StatFlags)\
	struct FStat_##StatName\
	{\
		using TGroup = UE_INTERNAL_GET_STATGROUP_TYPE(GroupName);\
		static constexpr const PROFILER_CHAR* GetName()\
		{\
			return ANSI_TO_PROFILING(#StatName);\
		}\
		static constexpr EStatFlags GetFlags()\
		{\
			return StatFlags;\
		}\
	};

// Replace declarations with lightweight instances
#define DECLARE_CYCLE_STAT(CounterName, StatId, GroupId) \
	DECLARE_STAT(StatId, GroupId, EStatFlags::None);

#define DECLARE_CYCLE_STAT_WITH_FLAGS(CounterName, StatId, GroupId, StatFlags) \
	DECLARE_STAT(StatId, GroupId, StatFlags);

#define DECLARE_CYCLE_STAT_EXTERN(CounterName, StatId, GroupId, APIX) \
	DECLARE_STAT(StatId, GroupId, EStatFlags::None);

#define DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(CounterName, StatId, GroupId, StatFlags, APIX) \
	DECLARE_STAT(StatId, GroupId, StatFlags);

#define DECLARE_MEMORY_STAT_EXTERN(CounterName, StatId, GroupId, API)\
	DECLARE_STAT(StatId, GroupId, EStatFlags::None);

#define DECLARE_MEMORY_STAT_POOL_EXTERN(CounterName, StatId, GroupId, Pool, API)\
	DECLARE_STAT(StatId, GroupId, EStatFlags::None);

#define DECLARE_MEMORY_STAT(CounterName,StatId,GroupId)\
	DECLARE_STAT(StatId, GroupId, EStatFlags::None);

#define DECLARE_MEMORY_STAT_POOL(CounterName,StatId,GroupId,Pool)\
	DECLARE_STAT(StatId, GroupId, EStatFlags::None);

// Some systems like the GPU profiler utilize counter declarations to declare stats used with SCOPE_CYCLE_COUNTER.
// Since this is a special case, allow falling back to the default stat data for these.
#define DECLARE_FLOAT_COUNTER_STAT(CounterName,StatId,GroupId)					UE_INTERNAL_ALLOW_DEFAULT_STAT(StatId);
#define DECLARE_FLOAT_COUNTER_STAT_EXTERN(CounterName,StatId,GroupId, API)		UE_INTERNAL_ALLOW_DEFAULT_STAT(StatId);
#define DECLARE_FLOAT_ACCUMULATOR_STAT(CounterName,StatId,GroupId)				UE_INTERNAL_ALLOW_DEFAULT_STAT(StatId);
#define DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(CounterName,StatId,GroupId, API)	UE_INTERNAL_ALLOW_DEFAULT_STAT(StatId);
#define DECLARE_DWORD_COUNTER_STAT(CounterName,StatId,GroupId)					UE_INTERNAL_ALLOW_DEFAULT_STAT(StatId);
#define DECLARE_DWORD_COUNTER_STAT_EXTERN(CounterName,StatId,GroupId, API)		UE_INTERNAL_ALLOW_DEFAULT_STAT(StatId);
#define DECLARE_DWORD_ACCUMULATOR_STAT(CounterName,StatId,GroupId)				UE_INTERNAL_ALLOW_DEFAULT_STAT(StatId);
#define DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(CounterName,StatId,GroupId, API)	UE_INTERNAL_ALLOW_DEFAULT_STAT(StatId);
#define DECLARE_FNAME_STAT(CounterName,StatId,GroupId, API)						UE_INTERNAL_ALLOW_DEFAULT_STAT(StatId);
#define DECLARE_PTR_STAT(CounterName,StatId,GroupId)							UE_INTERNAL_ALLOW_DEFAULT_STAT(StatId);
#define DECLARE_FNAME_STAT_EXTERN(CounterName,StatId,GroupId, API)				UE_INTERNAL_ALLOW_DEFAULT_STAT(StatId);
#define DECLARE_PTR_STAT_EXTERN(CounterName,StatId,GroupId, API)				UE_INTERNAL_ALLOW_DEFAULT_STAT(StatId);

// ----------------------------------------------------------------
// Scope Counters
// ----------------------------------------------------------------

#define SCOPE_CYCLE_COUNTER_TO_TRACE(StatString, StatName, Condition) \
	TRACE_CPUPROFILER_EVENT_DECLARE(UE_JOIN(__Decl_, StatName, __LINE__)); \
	TRACE_CPUPROFILER_EVENT_SCOPE_USE(UE_JOIN(__Decl_, StatName, __LINE__), StatString, UE_JOIN(__Scope_, StatName, __LINE__), Condition && GCycleStatsShouldEmitNamedEvents);

// Note: Since these are defining the stat inline, no custom stat data will exist.
#define DECLARE_SCOPE_CYCLE_COUNTER(CounterName,Stat,GroupId) \
	const UE::Stats::Private::FCheckedStat Stat_##Stat = UE_INTERNAL_GET_QUICK_STAT_WITH_GROUP_IF_ENABLED(Stat, GroupId);\
	UE::Stats::Private::FScopeCycleCounterStatic StatNamedEventsScope_##Stat(Stat_##Stat); \
	SCOPE_CYCLE_COUNTER_TO_TRACE(CounterName, Stat, Stat_##Stat.IsValidStat());

#define QUICK_SCOPE_CYCLE_COUNTER(Stat) \
	const UE::Stats::Private::FCheckedStat Stat_##Stat = UE_INTERNAL_GET_QUICK_STAT_IF_ENABLED(Stat);\
	UE::Stats::Private::FScopeCycleCounterStatic StatNamedEventsScope_##Stat(Stat_##Stat); \
	SCOPE_CYCLE_COUNTER_TO_TRACE(#Stat, Stat, Stat_##Stat.IsValidStat());

// Note: FScopeCycleCounter is what emits the trace events to external profilers
// while SCOPE_CYCLE_COUNTER_TO_TRACE handles emitting it to insights.
// FScopeCycleCounter handles verbosity checks itself.
#define SCOPE_CYCLE_COUNTER(Stat) \
	const UE::Stats::Private::FCheckedStat Stat_##Stat = UE_INTERNAL_GET_STAT_IF_ENABLED(Stat);\
	UE::Stats::Private::FScopeCycleCounterStatic StatNamedEventsScope_##Stat(Stat_##Stat); \
	SCOPE_CYCLE_COUNTER_TO_TRACE(#Stat, Stat, Stat_##Stat.IsValidStat());

// NOTE: This macro bypasses the standard enable checks as we don't know the stat's type/group.
#define SCOPE_CYCLE_COUNTER_STATID(StatId) \
	FScopeCycleCounter UE_JOIN(CycleCount_FromStatId_, __LINE__)(StatId); \
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(StatId.StatString, StatId.IsValidStat() && GCycleStatsShouldEmitNamedEvents > 0);

#define CONDITIONAL_SCOPE_CYCLE_COUNTER(Stat,bCondition) \
	const UE::Stats::Private::FCheckedStat Stat_##Stat = UE_INTERNAL_GET_STAT_IF_ENABLED_COND(Stat, static_cast<bool>(bCondition));\
	UE::Stats::Private::FScopeCycleCounterStatic StatNamedEventsScope_##Stat(Stat_##Stat); \
	SCOPE_CYCLE_COUNTER_TO_TRACE(#Stat, Stat, Stat_##Stat.IsValidStat());

#define RETURN_QUICK_DECLARE_CYCLE_STAT(StatId,GroupId) return QUICK_USE_CYCLE_STAT(StatId, GroupId);

#define QUICK_USE_CYCLE_STAT(StatId,GroupId) UE_INTERNAL_GET_QUICK_STAT_WITH_GROUP_IF_ENABLED(StatId, GroupId)

#define GET_STATID(Stat) UE_INTERNAL_GET_STAT_IF_ENABLED(Stat)

#elif USE_LIGHTWEIGHT_STATS_FOR_HITCH_DETECTION && USE_HITCH_DETECTION
// STATS == 0
// UE_USE_LIGHTWEIGHT_STATS == 0

// ----------------------------------------------------------------------------
// Hitch Detection
// ----------------------------------------------------------------------------
// This replaces the stat macros with a very lightweight scoped struct to create 
// a rough approximation of the current callstack for hitch tracking purposes.
// This let's us avoid traversing the actual callstack at runtime during a hitch which is very slow.
// ----------------------------------------------------------------------------

#define DECLARE_SCOPE_CYCLE_COUNTER(CounterName,Stat,GroupId) \
	UE::Stats::FHitchTrackingStatScope HitchTrackingStatScope_##Stat(ANSI_TO_PROFILING(#Stat));

#define QUICK_SCOPE_CYCLE_COUNTER(Stat) \
	UE::Stats::FHitchTrackingStatScope HitchTrackingStatScope_##Stat(ANSI_TO_PROFILING(#Stat));

#define SCOPE_CYCLE_COUNTER(Stat) \
	UE::Stats::FHitchTrackingStatScope HitchTrackingStatScope_##Stat(ANSI_TO_PROFILING(#Stat));

#define SCOPE_CYCLE_COUNTER_STATID(StatId) \
	UE::Stats::FHitchTrackingStatScope UE_JOIN(HitchTrackingStatScope_FromStatId_, __LINE__)(ANSI_TO_PROFILING("Lightweight StatId Scope"));

#define CONDITIONAL_SCOPE_CYCLE_COUNTER(Stat,bCondition) \
	UE::Stats::FHitchTrackingStatScope HitchTrackingStatScope_##Stat(bCondition ? ANSI_TO_PROFILING(#Stat) : nullptr);

#define RETURN_QUICK_DECLARE_CYCLE_STAT(StatId,GroupId) return TStatId();
#define GET_STATID(Stat) (TStatId())

#else
// STATS == 0
// UE_USE_LIGHTWEIGHT_STATS == 0
// (USE_LIGHTWEIGHT_STATS_FOR_HITCH_DETECTION && USE_HITCH_DETECTION) == 0

#define SCOPE_CYCLE_COUNTER(Stat)
#define SCOPE_CYCLE_COUNTER_STATID(StatId)
#define QUICK_SCOPE_CYCLE_COUNTER(Stat)
#define DECLARE_SCOPE_CYCLE_COUNTER(CounterName,StatId,GroupId)
#define CONDITIONAL_SCOPE_CYCLE_COUNTER(Stat,bCondition)
#define RETURN_QUICK_DECLARE_CYCLE_STAT(StatId,GroupId) return TStatId();
#define GET_STATID(Stat) (TStatId())
#endif // STATS

// ----------------------------------------------------------------------------
// Default Macro Definitions
// ----------------------------------------------------------------------------
#if !STATS
#define SCOPE_SECONDS_ACCUMULATOR(Stat)
#define SCOPE_MS_ACCUMULATOR(Stat)
#define DEFINE_STAT(Stat)

#if !UE_USE_LIGHTWEIGHT_STATS
#define QUICK_USE_CYCLE_STAT(StatId,GroupId) TStatId()
#define DECLARE_CYCLE_STAT(CounterName,StatId,GroupId)
#define DECLARE_CYCLE_STAT_WITH_FLAGS(CounterName,StatId,GroupId,StatFlags)
#define DECLARE_FLOAT_COUNTER_STAT(CounterName,StatId,GroupId)
#define DECLARE_DWORD_COUNTER_STAT(CounterName,StatId,GroupId)
#define DECLARE_FLOAT_ACCUMULATOR_STAT(CounterName,StatId,GroupId)
#define DECLARE_DWORD_ACCUMULATOR_STAT(CounterName,StatId,GroupId)
#define DECLARE_FNAME_STAT(CounterName,StatId,GroupId, API)
#define DECLARE_PTR_STAT(CounterName,StatId,GroupId)
#define DECLARE_MEMORY_STAT(CounterName,StatId,GroupId)
#define DECLARE_MEMORY_STAT_POOL(CounterName,StatId,GroupId,Pool)
#define DECLARE_CYCLE_STAT_EXTERN(CounterName,StatId,GroupId, API)
#define DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(CounterName,StatId,GroupId,StatFlags, API)
#define DECLARE_FLOAT_COUNTER_STAT_EXTERN(CounterName,StatId,GroupId, API)
#define DECLARE_DWORD_COUNTER_STAT_EXTERN(CounterName,StatId,GroupId, API)
#define DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(CounterName,StatId,GroupId, API)
#define DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(CounterName,StatId,GroupId, API)
#define DECLARE_FNAME_STAT_EXTERN(CounterName,StatId,GroupId, API)
#define DECLARE_PTR_STAT_EXTERN(CounterName,StatId,GroupId, API)
#define DECLARE_MEMORY_STAT_EXTERN(CounterName,StatId,GroupId, API)
#define DECLARE_MEMORY_STAT_POOL_EXTERN(CounterName,StatId,GroupId,Pool, API)
#define DECLARE_STATS_GROUP(GroupDesc,GroupId,GroupCat)
#define DECLARE_STATS_GROUP_VERBOSE(GroupDesc,GroupId,GroupCat)
#define DECLARE_STATS_GROUP_SORTBYNAME(GroupDesc,GroupId,GroupCat)
#define DECLARE_STATS_GROUP_MAYBE_COMPILED_OUT(GroupDesc,GroupId,GroupCat,CompileIn)
#endif // !UE_USE_LIGHTWEIGHT_STATS

#define SET_CYCLE_COUNTER(Stat,Cycles)
#define INC_DWORD_STAT(StatId)
#define INC_FLOAT_STAT_BY(StatId,Amount)
#define INC_DWORD_STAT_BY(StatId,Amount)
#define INC_DWORD_STAT_FNAME_BY(StatId,Amount)
#define INC_MEMORY_STAT_BY(StatId,Amount)
#define DEC_DWORD_STAT(StatId)
#define DEC_FLOAT_STAT_BY(StatId,Amount)
#define DEC_DWORD_STAT_BY(StatId,Amount)
#define DEC_DWORD_STAT_FNAME_BY(StatId,Amount)
#define DEC_MEMORY_STAT_BY(StatId,Amount)
#define SET_MEMORY_STAT(StatId,Value)
#define SET_DWORD_STAT(StatId,Value)
#define SET_FLOAT_STAT(StatId,Value)
#define STAT_ADD_CUSTOMMESSAGE_NAME(StatId,Value)
#define STAT_ADD_CUSTOMMESSAGE_PTR(StatId,Value)

#define SET_CYCLE_COUNTER_FName(Stat,Cycles)
#define INC_DWORD_STAT_FName(Stat)
#define INC_FLOAT_STAT_BY_FName(Stat, Amount)
#define INC_DWORD_STAT_BY_FName(Stat, Amount)
#define INC_MEMORY_STAT_BY_FName(Stat, Amount)
#define DEC_DWORD_STAT_FName(Stat)
#define DEC_FLOAT_STAT_BY_FName(Stat,Amount)
#define DEC_DWORD_STAT_BY_FName(Stat,Amount)
#define DEC_MEMORY_STAT_BY_FName(Stat,Amount)
#define SET_MEMORY_STAT_FName(Stat,Value)
#define SET_DWORD_STAT_FName(Stat,Value)
#define SET_FLOAT_STAT_FName(Stat,Value)

#define GET_STATFNAME(Stat) (FName())
#define GET_STATDESCRIPTION(Stat) (nullptr)
#endif // !STATS

#include "Stats/GlobalStats.inl"
