// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.6, "All definitions have been moved to Stats.h, please use that instead.")

#include "Stats/Stats.h"

