// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"
#include "Misc/CoreMiscDefines.h"

class FOutputDevice;

namespace UE::Stats
{
	// Pass a console command directly to the stats system, return true if it is known command, false means it might be a stats command
	CORE_API bool DirectStatsCommand(const TCHAR* Cmd, bool bBlockForCompletion = false, FOutputDevice* Ar = nullptr);
}
