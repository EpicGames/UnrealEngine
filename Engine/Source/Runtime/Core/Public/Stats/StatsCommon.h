// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Build.h"
#include "Misc/EnumClassFlags.h"

#define FORCEINLINE_STATS FORCEINLINE
//#define FORCEINLINE_STATS FORCEINLINE_DEBUGGABLE
#define checkStats(x)

#if !defined(STATS)
#error "STATS must be defined as either zero or one."
#endif

#ifndef UE_USE_LIGHTWEIGHT_STATS
#define UE_USE_LIGHTWEIGHT_STATS (!STATS && ENABLE_STATNAMEDEVENTS)
#endif

#if	PLATFORM_USES_ANSI_STRING_FOR_EXTERNAL_PROFILING
typedef ANSICHAR PROFILER_CHAR;
#else
typedef WIDECHAR PROFILER_CHAR;
#endif // PLATFORM_USES_ANSI_STRING_FOR_EXTERNAL_PROFILING

#if PLATFORM_USES_ANSI_STRING_FOR_EXTERNAL_PROFILING
#define ANSI_TO_PROFILING(x) x
#else
#define ANSI_TO_PROFILING(x) TEXT(x)
#endif

enum class EStatFlags : uint8
{
	None = 0,
	ClearEveryFrame = 1 << 0,
	CycleStat = 1 << 1,
	Verbose = 1 << 2, // Profiling scopes for this stat will no generate a trace event by default. See GShouldEmitVerboseNamedEvents.
};
ENUM_CLASS_FLAGS(EStatFlags);