// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMemory.h"
#include "HAL/ThreadSafeCounter.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Build.h"
#include "Misc/Parse.h"
#include "Stats/Stats.h"
#include "Stats/StatsCommon.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

struct FEventData;

#if STATS

/**
* Used for a few different things, but roughly one more than the maximum render thread game thread lag, in frames.
*/
enum
{
	STAT_FRAME_SLOP = 3,
	MAX_STAT_LAG = 4,
};

/** Holds stats related constants. */
struct FStatConstants
{
	/** Special name for thread root. */
	static CORE_API const FName NAME_ThreadRoot;

	/** This is a special group name used to store threads metadata. */
	static CORE_API const char* ThreadGroupName;
	static CORE_API const FName NAME_ThreadGroup;

	/** Stat raw name for seconds per cycle. */
	static CORE_API const FName RAW_SecondsPerCycle;

	/** Special case category, when we want to Stat to appear at the root of the menu (leaving the category blank omits it from the menu entirely) */
	static CORE_API const FName NAME_NoCategory;

	/** Extension used to save a stats file. */
	static CORE_API const FString StatsFileExtension;

	/** Extension used to save a raw stats file, may be changed to the same as a regular stats file. */
	static CORE_API const FString StatsFileRawExtension;

	/** Indicates that the item is a thread. */
	static CORE_API const FString ThreadNameMarker;

	/** A raw name for the event wait with id. */
	static CORE_API const FName RAW_EventWaitWithId;

	/** A raw name for the event trigger with id. */
	static CORE_API const FName RAW_EventTriggerWithId;

	/** A raw name for the stat marker. */
	static CORE_API const FName RAW_NamedMarker;

	/** A special meta data used to advance the frame. */
	static CORE_API const FStatNameAndInfo AdvanceFrame;
};

/** Parse a typed value into the specified out parameter.
 * 	Expects to find a FromString function that takes a reference to T. Defaults are provided in the Lex namespace.
 */
template <typename T>
void ParseTypedValue( const TCHAR* Stream, const TCHAR* Match, T& Out )
{
	TCHAR Temp[64] = TEXT( "" );
	if( FParse::Value( Stream, Match, Temp, UE_ARRAY_COUNT( Temp ) ) )
	{
		LexFromString( Out, Temp );
	}
}

/**
*	Helper class to parse a value from the stream.
*	If value is not present, provided default value will be used.
*/
template<typename T>
struct TParsedValueWithDefault
{
public:
	TParsedValueWithDefault( const TCHAR* Stream, const TCHAR* Match, const T& Default )
		: Value( Default )
	{
		if (Stream && Match)
		{
			ParseTypedValue<T>( Stream, Match, Value );
		}
	}

	const T& Get() const
	{
		return Value;
	}

	void Set( const T& NewValue )
	{
		Value = NewValue;
	}

private:
	T Value;
};

/** Enumerates stat compare types. */
struct EStatCompareBy
{
	enum Type
	{
		/** By stat name. */
		Name,
		/** By call count, only for scoped cycle counters. */
		CallCount,
		/** By total accumulated value. */
		Sum,
	};
};

/** Stat display mode. */
struct EStatDisplayMode
{
	enum Type
	{
		Invalid			= 0x0,
		Hierarchical	= 0x1,
		Flat			= 0x2,
	};
};

/** Sort predicate for alphabetic ordering. */
template< typename T >
struct FStatNameComparer
{
	inline bool operator()( const T& A, const T& B ) const
	{
		check(0);
		return 0;
	}
};

/** Sort predicate with the slowest inclusive time first. */
template< typename T >
struct FStatDurationComparer
{
	inline bool operator()( const T& A, const T& B ) const
	{
		check(0);
		return 0;
	}
};

/** Sort predicate with the lowest call count first.  */
template< typename T >
struct FStatCallCountComparer
{
	inline bool operator()( const T& A, const T& B ) const
	{
		check(0);
		return 0;
	}
};

/** Sort predicate for alphabetic ordering, specialized for FStatMessage. */
template<>
struct FStatNameComparer<FStatMessage>
{
	UE_FORCEINLINE_HINT bool operator()( const FStatMessage& A, const FStatMessage& B ) const
	{
		return A.NameAndInfo.GetRawName().Compare( B.NameAndInfo.GetRawName() ) < 0;
	}
};

/** Sort predicate with the slowest inclusive time first, specialized for FStatMessage. */
template<>
struct FStatDurationComparer<FStatMessage>
{
	inline bool operator()( const FStatMessage& A, const FStatMessage& B ) const
	{
		const uint32 DurationA = FromPackedCallCountDuration_Duration( A.GetValue_int64() );
		const uint32 DurationB = FromPackedCallCountDuration_Duration( B.GetValue_int64() );
		return DurationA == DurationB ? FStatNameComparer<FStatMessage>()(A,B) : DurationA > DurationB;
	}
};

/** Sort predicate with the lowest call count first, specialized for FStatMessage. */
template<>
struct FStatCallCountComparer<FStatMessage>
{
	inline bool operator()( const FStatMessage& A, const FStatMessage& B ) const
	{
		const uint32 CallCountA = FromPackedCallCountDuration_CallCount( A.GetValue_int64() );
		const uint32 CallCountB = FromPackedCallCountDuration_CallCount( B.GetValue_int64() );
		return CallCountA == CallCountB ? FStatNameComparer<FStatMessage>()(A,B) : CallCountA > CallCountB;
	}
};

/**
* An indirect array of stat packets.
*/
//@todo can we just use TIndirectArray here?
struct FStatPacketArray
{
	TArray<FStatPacket*> Packets;

	~FStatPacketArray()
	{
		Empty();
	}

	/** Deletes all stats packets. */
	CORE_API void Empty();

	void RemovePtrsButNoData()
	{
		Packets.Empty();
	}
};

/**
* A call stack of stat messages. Normally we don't store or transmit these; they are used for stats processing and visualization.
*/
struct FRawStatStackNode
{
	/** The stat message corresponding to this node **/
	FStatMessage Meta; // includes aggregated inclusive time and call counts packed into the int64

	/** Map from long name to children of this node **/
	TMap<FName, FRawStatStackNode*> Children;

	/** Constructor, this always and only builds the thread root node. The thread root is not a numeric stat! **/
	FRawStatStackNode()
		: Meta(FStatConstants::NAME_ThreadRoot, EStatDataType::ST_None, nullptr, nullptr, nullptr, false, false, false)
	{
	}

	/** Copy Constructor **/
	CORE_API explicit FRawStatStackNode(FRawStatStackNode const& Other);

	/** Constructor used to build a child from a stat messasge **/
	explicit FRawStatStackNode(FStatMessage const& InMeta)
		: Meta(InMeta)
	{
	}

	/** Assignment operator. */
	FRawStatStackNode& operator=(const FRawStatStackNode& Other)
	{
		if( this != &Other )
		{
			DeleteAllChildrenNodes();
			Meta = Other.Meta;

			Children.Empty(Other.Children.Num());
			for (TMap<FName, FRawStatStackNode*>::TConstIterator It(Other.Children); It; ++It)
			{
				Children.Add(It.Key(), new FRawStatStackNode(*It.Value()));
			}
		}
		return *this;
	}

	/** Destructor, clean up the memory and children. **/
	~FRawStatStackNode()
	{
		DeleteAllChildrenNodes();
	}

	CORE_API void MergeMax(FRawStatStackNode const& Other);
	CORE_API void MergeAdd(FRawStatStackNode const& Other);
	CORE_API void Divide(uint32 Div);

	/** Cull this tree, merging children below MinCycles long. */
	CORE_API void CullByCycles(int64 MinCycles);

	/** Cull this tree, merging children below NoCullLevels. */
	CORE_API void CullByDepth(int32 NoCullLevels);

	/** Adds name hiearchy. **/
	CORE_API void AddNameHierarchy(int32 CurrentPrefixDepth = 0);

	/** Adds self nodes. **/
	CORE_API void AddSelf();

	/** Print this tree to the log **/
	CORE_API void DebugPrint(TCHAR const* Filter = nullptr, int32 MaxDepth = MAX_int32, int32 Depth = 0) const;

	/** Print this tree to the log **/
	CORE_API void DebugPrintLeafFilter(TCHAR const* Filter) const;

	/** Print this tree to the log **/
	CORE_API void DebugPrintLeafFilterInner(TCHAR const* Filter, int32 Depth, TArray<FString>& Stack) const;

	/** Condense this tree into a flat list using EStatOperation::ChildrenStart, EStatOperation::ChildrenEnd, EStatOperation::Leaf **/
	CORE_API void Encode(TArray<FStatMessage>& OutStats) const;

	/** Sum the inclusive cycles of the children **/
	CORE_API int64 ChildCycles() const;

	/** Sorts children using the specified comparer class. */
	template< class TComparer >
	void Sort( const TComparer& Comparer )
	{
		Children.ValueSort( Comparer );
		for( auto It = Children.CreateIterator(); It; ++It )
		{
			FRawStatStackNode* Child = It.Value();
			Child->Sort( Comparer );
		}
	}

	/** Custom new/delete; these nodes are always pooled, except the root which one normally puts on the stack. */
	CORE_API void* operator new(size_t Size);
	CORE_API void operator delete(void *RawMemory);

protected:
	void DeleteAllChildrenNodes()
	{
		for (TMap<FName, FRawStatStackNode*>::TIterator It(Children); It; ++It)
		{
			delete It.Value();
		}
	}
};

/** Sort predicate for alphabetic ordering, specialized for FRawStatStackNode. */
template<>
struct FStatNameComparer<FRawStatStackNode>
{
	UE_FORCEINLINE_HINT bool operator()( const FRawStatStackNode& A, const FRawStatStackNode& B ) const
	{
		return A.Meta.NameAndInfo.GetRawName().Compare( B.Meta.NameAndInfo.GetRawName() ) < 0;
	}
};

/** Sort predicate with the slowest inclusive time first, specialized for FRawStatStackNode. */
template<>
struct FStatDurationComparer<FRawStatStackNode>
{
	inline bool operator()( const FRawStatStackNode& A, const FRawStatStackNode& B ) const
	{
		const uint32 DurationA = FromPackedCallCountDuration_Duration( A.Meta.GetValue_int64() );
		const uint32 DurationB = FromPackedCallCountDuration_Duration( B.Meta.GetValue_int64() );
		return DurationA == DurationB ? FStatNameComparer<FRawStatStackNode>()(A,B) : DurationA > DurationB;
	}
};

/** Sort predicate with the lowest call count first, specialized for FRawStatStackNode. */
template<>
struct FStatCallCountComparer<FRawStatStackNode>
{
	inline bool operator()( const FRawStatStackNode& A, const FRawStatStackNode& B ) const
	{
		const uint32 CallCountA = FromPackedCallCountDuration_CallCount( A.Meta.GetValue_int64() );
		const uint32 CallCountB = FromPackedCallCountDuration_CallCount( B.Meta.GetValue_int64() );
		return CallCountA == CallCountB ? FStatNameComparer<FRawStatStackNode>()(A,B) : CallCountA > CallCountB;
	}
};

/** Same as the FRawStatStackNode, but for the FComplexStatMessage. */
struct FComplexRawStatStackNode
{
	/** Complex stat. */
	FComplexStatMessage ComplexStat;

	/** Map from long name to children of this node **/
	TMap<FName, FComplexRawStatStackNode*> Children;

	/** Default constructor. */
	FComplexRawStatStackNode()
	{}

	/** Copy constructor. */
	explicit FComplexRawStatStackNode(FComplexRawStatStackNode const& Other);

	/** Copy constructor. */
	explicit FComplexRawStatStackNode(FRawStatStackNode const& Other);

	void CopyNameHierarchy( const FRawStatStackNode& Other )	
	{
		DeleteAllChildrenNodes();

		ComplexStat = Other.Meta;

		Children.Empty(Other.Children.Num());
		for (auto It = Other.Children.CreateConstIterator(); It; ++It)
		{
			Children.Add(It.Key(), new FComplexRawStatStackNode(*It.Value()));
		}
	}

public:
	/** Destructor, clean up the memory and children. **/
	~FComplexRawStatStackNode()
	{
		DeleteAllChildrenNodes();
	}

	/** Merges this stack with the other. */
	void MergeAddAndMinMax( const FRawStatStackNode& Other );

	/** Divides this stack by the specified value. */
	void Divide(uint32 Div);

	/** Cull this tree, merging children below MinCycles long. */
	void CullByCycles( int64 MinCycles );

	/** Cull this tree, merging children below NoCullLevels. */
	void CullByDepth( int32 NoCullLevels );

	/** Copies exclusive times from the self node. **/
	void CopyExclusivesFromSelf();

	/** Custom new/delete; these nodes are always pooled, except the root which one normally puts on the stack. */
	void* operator new(size_t Size);
	void operator delete(void *RawMemory);

protected:
	void DeleteAllChildrenNodes()
	{
		for (auto It = Children.CreateIterator(); It; ++It)
		{
			delete It.Value();
		}
	}
};


/** Holds information about event history, callstacks for the wait and the corresponding trigger. */
struct FEventData
{
	/** Default constructor. */
	FEventData()
		: Frame( 0 )
		, Duration( 0 )
		, DurationMS( 0.0f )
	{}

	/**
	 * @return true, if the wait and trigger stacks are not empty.
	 */
	bool HasValidStacks() const
	{
		return WaitStackStats.Num() > 0 && TriggerStackStats.Num() > 0;
	}

	/** Stack for the event's wait. */
	TArray<FStatNameAndInfo> WaitStackStats;
	/** Stat for the event's trigger. */
	TArray<FStatNameAndInfo> TriggerStackStats;
	/** Frame, used to remove older records from the history. */
	int64 Frame;
	/** Duration of the event's wait, in cycles. */
	uint32 Duration;
	/** Duration of the event's wait, in milliseconds. */
	float DurationMS;
};

//@todo split header

/**
* Some of the FStatsThreadState data methods allow filtering. Derive your filter from this class
*/
struct IItemFilter
{
	virtual ~IItemFilter() = default;

	/** return true if you want to keep the item for the purposes of collecting stats **/
	virtual bool Keep(FStatMessage const& Item) = 0;
};


// #Stats: 2014-03-21 Move metadata functionality into a separate class
// 
/**
 * Tracks stat state and history
 * GetLocalState() is a singleton to the state for stats being collected in this executable.
 */
class FStatsThreadState
{
	friend class FStatsThread;
	friend struct FStatPacketArray;
	friend struct FStatsReadFile;

	/** Delegate that FStatsThreadState calls on the stats thread whenever we have a new frame. */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnNewFrameHistory, int64);

	/** Delegate that FStatsThreadState calls on the stats thread whenever we have a new raw stats packet. */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnNewRawStatPacket, const FStatPacket*);

	/** Internal method to scan the messages to update the current frame. */
	CORE_API void ScanForAdvance(const FStatMessagesArray& Data);

	/** Internal method to scan the messages to update the current frame. */
	CORE_API void ScanForAdvance(FStatPacketArray& NewData);

public:
	CORE_API void ProcessMetaDataOnly(TArray64<FStatMessage>& Data);

	/** Toggles tracking the most memory expensive stats. */
	CORE_API void ToggleFindMemoryExtensiveStats();

	/** Resets stats for raw stats. */
	void ResetStatsForRawStats()
	{
		MaxNumStatMessages = 0;
		TotalNumStatMessages = 0;
	}

private:
	/** Internal method to accumulate any non-frame stats. **/
	CORE_API void ProcessNonFrameStats( FStatMessagesArray& Data, TSet<FName>* NonFrameStatsFound );

	/** Internal method to place the data into the history, discard and broadcast any new frames to anyone who cares. **/
	CORE_API void AddToHistoryAndEmpty( FStatPacketArray& NewData );

	/** Does basic processing on the raw stats packets, discard and broadcast any new raw stats packets to anyone who cares. */
	CORE_API void ProcessRawStats(FStatPacketArray& NewData);

	/** Resets the state of the raw stats. */
	CORE_API void ResetRawStats();

	/** Reset the state of the regular stats. */
	CORE_API void ResetRegularStats();

	/** Prepares fake FGameThreadStatsData to display the raw stats memory overhead. */
	CORE_API void UpdateStatMessagesMemoryUsage();

	/** Generates a list of most memory expensive stats and dump to the log. */
	CORE_API void FindAndDumpMemoryExtensiveStats( const FStatPacketArray& Frame );

protected:
	/** Called in response to SetLongName messages to update ShortNameToLongName and NotClearedEveryFrame **/
	CORE_API void FindOrAddMetaData(FStatMessage const& Item);

	/** Parse the raw frame history into a tree representation **/
	CORE_API void GetRawStackStats(int64 FrameNumber, FRawStatStackNode& Out, TArray<FStatMessage>* OutNonStackStats = nullptr) const;

	/** Parse the raw frame history into a flat-array-based tree representation **/
	CORE_API void Condense(int64 TargetFrame, TArray<FStatMessage>& OutStats) const;

	/** Returns cycles spent in the specified thread. */
	CORE_API int64 GetFastThreadFrameTimeInternal(int64 TargetFrame, int32 ThreadID, EThreadType::Type Thread) const;

	/** Number of frames to keep in the history. **/
	int32 HistoryFrames;

	/** Used to track which packets have been sent to listeners **/
	int64 LastFullFrameMetaAndNonFrame;

	/** Used to track which packets have been sent to listeners **/
	int64 LastFullFrameProcessed;

	/** Cached condensed frames. This saves a lot of time since multiple listeners can use the same condensed data **/
	mutable TMap<int64, TArray<FStatMessage>* > CondensedStackHistory;

	/** After we add to the history, these frames are known good. **/
	TSet<int64>	GoodFrames;

	/** These frames are known bad. **/
	TSet<int64>	BadFrames;

	/** Array of stat packets that are not processed yet. */
	FStatPacketArray StartupRawStats;

	/** Total number of stats messages for active session. */
	int64 TotalNumStatMessages;

	/** Current number of stat messages stored by stats thread and ready to be processed (for visualization, for saving). */
	FThreadSafeCounter NumStatMessages;

	/** Maximum number of stat messages seen so far for active session. */
	int32 MaxNumStatMessages;

	/** If true, stats each frame will dump to the log a list of most memory expensive stats. */
	bool bFindMemoryExtensiveStats;

public:

	/** Constructor used by GetLocalState(), also used by the profiler to hold a previewing stats thread state. We don't keep many frames by default **/
	CORE_API FStatsThreadState(int32 InHistoryFrames = STAT_FRAME_SLOP + 10);

	/** Delegate we fire every time we have a new complete frame of data. **/
	mutable FOnNewFrameHistory NewFrameDelegate;

	/** Delegate we fire every time we have a new raw stats packet, the data is no longer valid outside the broadcast. */
	mutable FOnNewRawStatPacket NewRawStatPacket;

	/** New packets not on the render thread as assign this frame **/
	int64 CurrentGameFrame;

	/** New packets on the render thread as assign this frame **/
	int64 CurrentRenderFrame;

	int64 CurrentEndOfPipeFrame;

	/** List of the stats, with data of all of the things that are NOT cleared every frame. We need to accumulate these over the entire life of the executable. **/
	TMap<FName, FStatMessage> NotClearedEveryFrame;

	/** Map from a short name to the SetLongName message that defines the metadata and long name for this stat **/
	TMap<FName, FStatMessage> ShortNameToLongName;

	/** Map from the unique event id to the event stats. */
	mutable TMap<uint32, FEventData> EventsHistory;

	/** Map from memory pool to long name**/
	TMap<FPlatformMemory::EMemoryCounterRegion, FName> MemoryPoolToCapacityLongName;

	/** Defines the groups. the group called "Groups" can be used to enumerate the groups. **/
	TMultiMap<FName, FName> Groups;

	/** Map from a thread id to the thread name. */
	TMap<uint32, FName> Threads;

	/** Raw data for a frame. **/
	TMap<int64, FStatPacketArray> History;

	/** Return the oldest frame of data we have. **/
	CORE_API int64 GetOldestValidFrame() const;

	/** Return the newest frame of data we have. **/
	CORE_API int64 GetLatestValidFrame() const;

	/** Return true if this is a valid frame. **/
	bool IsFrameValid(int64 Frame) const
	{
		return GoodFrames.Contains(Frame);
	}

	/** Used by the hitch detector. Does a very fast look a thread to decide if this is a hitch frame. **/
	CORE_API int64 GetFastThreadFrameTime(int64 TargetFrame, EThreadType::Type Thread) const;
	CORE_API int64 GetFastThreadFrameTime(int64 TargetFrame, uint32 ThreadID) const;

	/** For the specified stat packet looks for the thread name. */
	CORE_API FName GetStatThreadName( const FStatPacket& Packet ) const;

	/** Looks in the history for a condensed frame, builds it if it isn't there. **/
	CORE_API TArray<FStatMessage> const& GetCondensedHistory(int64 TargetFrame) const;

	/** Looks in the history for a stat packet array, the stat packet array must be in the history. */
	const FStatPacketArray& GetStatPacketArray( int64 TargetFrame ) const
	{
		check( IsFrameValid( TargetFrame ) );
		const FStatPacketArray& Frame = History.FindChecked( TargetFrame );
		return Frame;
	}

	/** Gets the old-skool flat grouped inclusive stats. These ignore recursion, merge threads, etc and so generally the condensed callstack is less confusing. **/
	CORE_API void GetInclusiveAggregateStackStats( const TArray<FStatMessage>& CondensedMessages, TArray<FStatMessage>& OutStats, IItemFilter* Filter = nullptr, bool bAddNonStackStats = true, TMap<FName, TArray<FStatMessage>>* OptionalOutThreadBreakdownMap = nullptr ) const;

	/** Gets the old-skool flat grouped inclusive stats. These ignore recursion, merge threads, etc and so generally the condensed callstack is less confusing. **/
	CORE_API void GetInclusiveAggregateStackStats(int64 TargetFrame, TArray<FStatMessage>& OutStats, IItemFilter* Filter = nullptr, bool bAddNonStackStats = true, TMap<FName, TArray<FStatMessage>>* OptionalOutThreadBreakdownMap = nullptr) const;

	/** Gets the old-skool flat grouped exclusive stats. These merge threads, etc and so generally the condensed callstack is less confusing. **/
	CORE_API void GetExclusiveAggregateStackStats( const TArray<FStatMessage>& CondensedMessages, TArray<FStatMessage>& OutStats, IItemFilter* Filter = nullptr, bool bAddNonStackStats = true ) const;

	/** Gets the old-skool flat grouped exclusive stats. These merge threads, etc and so generally the condensed callstack is less confusing. **/
	CORE_API void GetExclusiveAggregateStackStats(int64 TargetFrame, TArray<FStatMessage>& OutStats, IItemFilter* Filter = nullptr, bool bAddNonStackStats = true) const;

	/** Used to turn the condensed version of stack stats back into a tree for easier handling. **/
	CORE_API void UncondenseStackStats( const TArray<FStatMessage>& CondensedMessages, FRawStatStackNode& Root, IItemFilter* Filter = nullptr, TArray<FStatMessage>* OutNonStackStats = nullptr ) const;

	/** Used to turn the condensed version of stack stats back into a tree for easier handling. **/
	CORE_API void UncondenseStackStats(int64 TargetFrame, FRawStatStackNode& Root, IItemFilter* Filter = nullptr, TArray<FStatMessage>* OutNonStackStats = nullptr) const;

	/** Adds missing stats to the group so it doesn't jitter. **/
	CORE_API void AddMissingStats(TArray<FStatMessage>& Dest, TSet<FName> const& EnabledItems) const;

	/** Singleton to get the stats being collected by this executable. Can be only accessed from the stats thread. **/
	static CORE_API FStatsThreadState& GetLocalState();
};

//@todo split header

/**
* Set of utility functions for dealing with stats
*/
struct FStatsUtils
{
	/** Divides a stat by an integer. Used to finish an average. **/
	static CORE_API void DivideStat(FStatMessage& Dest, uint32 Div);

	/** Merges two stat message arrays, adding corresponding number values **/
	static CORE_API void AddMergeStatArray(TArray<FStatMessage>& Dest, TArray<FStatMessage> const& Item);

	/** Merges two stat message arrays, corresponding number values are set to the max **/
	static CORE_API void MaxMergeStatArray(TArray<FStatMessage>& Dest, TArray<FStatMessage> const& Item);

	/** Divides a stat array by an integer. Used to finish an average. **/
	static CORE_API void DivideStatArray(TArray<FStatMessage>& Dest, uint32 Div);

	/** Apply Op or Item.Op to Item and Dest, soring the result in dest. **/
	static CORE_API void AccumulateStat(FStatMessage& Dest, FStatMessage const& Item, EStatOperation::Type Op = EStatOperation::Invalid, bool bAllowNameMismatch = false);

	/** Adds a non-stack stats */
	static void AddNonStackStats( const FName LongName, const FStatMessage& Item, const EStatOperation::Type Op, TMap<FName, FStatMessage>& out_NonStackStats )
	{
		if (
			Item.NameAndInfo.GetField<EStatDataType>() != EStatDataType::ST_None && 
			Item.NameAndInfo.GetField<EStatDataType>() != EStatDataType::ST_FName &&
			(Op == EStatOperation::Set ||
			Op == EStatOperation::Clear ||
			Op == EStatOperation::Add ||
			Op == EStatOperation::Subtract ||
			Op == EStatOperation::MaxVal))
		{
			FStatMessage* Result = out_NonStackStats.Find(LongName);
			if (!Result)
			{
				Result = &out_NonStackStats.Add(LongName, Item);
				Result->NameAndInfo.SetField<EStatOperation>(EStatOperation::Set);
				Result->Clear();
			}
			FStatsUtils::AccumulateStat(*Result, Item);
		}
	}

	/** Spews a stat message, not all messages are supported, this is used for reports, so it focuses on numeric output, not say the operation. **/
	static CORE_API FString DebugPrint(FStatMessage const& Item);

	/** Gets friendly names from a stat message, not all messages are supported, this is used for reports, **/
	static CORE_API void GetNameAndGroup(FStatMessage const& Item, FString& OutName, FString& OutGroup);

	/** Subtract a CycleScopeStart from a CycleScopeEnd to create a IsPackedCCAndDuration with the call count and inclusive cycles. **/
	static FStatMessage ComputeCall(FStatMessage const& ScopeStart, FStatMessage const& ScopeEnd)
	{
		checkStats(ScopeStart.NameAndInfo.GetField<EStatOperation>() == EStatOperation::CycleScopeStart);
		checkStats(ScopeEnd.NameAndInfo.GetField<EStatOperation>() == EStatOperation::CycleScopeEnd);
		FStatMessage Result(ScopeStart);
		Result.NameAndInfo.SetField<EStatOperation>(EStatOperation::Set);
		Result.NameAndInfo.SetFlag(EStatMetaFlags::IsPackedCCAndDuration, true);
		checkStats(ScopeEnd.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle));

		// cycles can wrap and are actually uint32's so we do something special here
		int64 Delta = int32(uint32(ScopeEnd.GetValue_int64()) - uint32(ScopeStart.GetValue_int64()));
		checkStats(Delta >= 0);
		Result.GetValue_int64() = ToPackedCallCountDuration(1, uint32(Delta));
		return Result;
	}

	/** Finds a maximum for int64 based stat data. */
	static void StatOpMaxVal_Int64( const FStatNameAndInfo& DestNameAndInfo, int64& Dest, const int64& Other )
	{
		if (DestNameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration))
		{
			Dest = ToPackedCallCountDuration(
				FMath::Max<uint32>(FromPackedCallCountDuration_CallCount(Dest), FromPackedCallCountDuration_CallCount(Other)),
				FMath::Max<uint32>(FromPackedCallCountDuration_Duration(Dest), FromPackedCallCountDuration_Duration(Other)));
		}
		else
		{
			Dest = FMath::Max<int64>(Dest,Other);
		}
	}
	/** Finds a minimum for int64 based stat data. */
	static void StatOpMinVal_Int64(const FStatNameAndInfo& DestNameAndInfo, int64& Dest, const int64& Other)
	{
		if (DestNameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration))
		{
			Dest = ToPackedCallCountDuration(
				FMath::Min<uint32>(FromPackedCallCountDuration_CallCount(Dest), FromPackedCallCountDuration_CallCount(Other)),
				FMath::Min<uint32>(FromPackedCallCountDuration_Duration(Dest), FromPackedCallCountDuration_Duration(Other)));
		}
		else
		{
			Dest = FMath::Min<int64>(Dest, Other);
		}
	}

	/** Internal use, converts arbitrary string to and from an escaped notation for storage in an FName. **/
	static CORE_API void ToEscapedString(FStringView Input, FStringBuilderBase& Output);
	static CORE_API FString FromEscapedString(const TCHAR* Escaped);
	
	static FString BuildUniqueThreadName( uint32 InThreadID )
	{
		// Make unique name.
		return FString::Printf( TEXT( "%s%x_0" ), *FStatConstants::ThreadNameMarker, InThreadID );
	}

	static int32 ParseThreadID( const FString& InThreadName, FString* out_ThreadName = nullptr )
	{
		// Extract the thread id.
		const FString ThreadName = InThreadName.Replace( TEXT( "_0" ), TEXT( "" ) );
		int32 Index = 0;
		ThreadName.FindLastChar(TEXT('_'),Index);

		// Parse the thread name if requested.
		if( out_ThreadName )
		{
			*out_ThreadName = ThreadName.Left( Index );
		}

		const FString ThreadIDStr = ThreadName.RightChop(Index+1);
		const uint32 ThreadID = FParse::HexNumber( *ThreadIDStr );

		return ThreadID;
	}
};

/**
* Contains helpers functions to manage complex stat messages.
*/
struct FComplexStatUtils
{
	/** Accumulates a stat message into a complex stat message. */
	static CORE_API void AddAndMinMax( FComplexStatMessage& Dest, const FStatMessage& Item, EComplexStatField::Type SumIndex, EComplexStatField::Type MaxIndex, EComplexStatField::Type MinIndex);

	/** Divides a complex stat message by the specified value. */
	static CORE_API void DivideStat( FComplexStatMessage& Dest, uint32 Div, EComplexStatField::Type SumIndex, EComplexStatField::Type DestIndex );

	/** Merges two complex stat message arrays. */
	static CORE_API void MergeAddAndMinMaxArray( TArray<FComplexStatMessage>& Dest, const TArray<FStatMessage>& Source, EComplexStatField::Type SumIndex, EComplexStatField::Type MaxIndex, EComplexStatField::Type MinIndex);

	/** Divides a complex stat array by the specified value. */
	static CORE_API void DiviveStatArray( TArray<FComplexStatMessage>& Dest, uint32 Div, EComplexStatField::Type SumIndex, EComplexStatField::Type DestIndex );
};

//@todo split header

/** Holds stats data used by various systems like the HUD stats*/
struct FActiveStatGroupInfo
{
	/** Array of all flat aggregates for the last n frames. */
	TArray<FComplexStatMessage> FlatAggregate;

	/** Array of all flat aggregates for the last n frames broken down by thread. */
	TMap<FName, TArray<FComplexStatMessage>> FlatAggregateThreadBreakdown;
	
	/** Array of all aggregates for the last n frames. */
	TArray<FComplexStatMessage> HierAggregate;

	/** Array of indentations for the history stack, must match the size. */
	TArray<int32> Indentation;

	/** Memory aggregates. */
	TArray<FComplexStatMessage> MemoryAggregate;

	/** Counters aggregates. */
	TArray<FComplexStatMessage> CountersAggregate;

	/** GPU aggregates. */
	TArray<FComplexStatMessage> GpuStatsAggregate;

	/** Children stats that should not be used when adding up group cost **/
	TSet<FName> BudgetIgnoreStats;

	/** Expected group budget */
	TMap<FName, float> ThreadBudgetMap;
};

/**
* Information sent from the stats thread to the game thread to render and be used by other systems
*/
struct FGameThreadStatsData
{
	/** Initialization constructor. */
	FGameThreadStatsData( bool bInDrawOnlyRawStats, bool bInRenderStats )
		: bDrawOnlyRawStats(bInDrawOnlyRawStats)
		, bRenderStats(bInRenderStats)
	{}

	/** NOTE: the returned pointer is only valid for this frame - do not hold on to it! */
	const FComplexStatMessage* GetStatData(const FName& StatName ) const
	{
		return NameToStatMap.FindRef(StatName);
	}

	TIndirectArray<FActiveStatGroupInfo> ActiveStatGroups;
	TArray<FName> GroupNames;
	TArray<FString> GroupDescriptions;
	TMap<FPlatformMemory::EMemoryCounterRegion, int64> PoolCapacity;
	TMap<FPlatformMemory::EMemoryCounterRegion, FString> PoolAbbreviation;
	FString RootFilter;

	TMap<FName, const FComplexStatMessage*> NameToStatMap;

	/** Whether to display minimal stats for the raw stats mode. */
	const bool bDrawOnlyRawStats;

	/** Whether to render the stats to HUD */
	const bool bRenderStats;
};

/**
* Singleton that holds the last data sent from the stats thread to the game thread for systems to use and display
*/
struct FLatestGameThreadStatsData 
{
	FGameThreadStatsData* Latest;

	FLatestGameThreadStatsData()
		: Latest(nullptr)
	{
	}

	~FLatestGameThreadStatsData()
	{
		delete Latest;
		Latest = nullptr;
	}

	void NewData(FGameThreadStatsData* Data);

	CORE_API static FLatestGameThreadStatsData& Get();
};

/**
 * Singleton that holds a list of newly registered group stats to inform the game thread of
 */
struct FStatGroupGameThreadNotifier
{
public:
	CORE_API static FStatGroupGameThreadNotifier& Get();

	void NewData(FStatNameAndInfo NameAndInfo)
	{
		NameAndInfos.Add(NameAndInfo);
		const FName GroupName = NameAndInfo.GetGroupName();
		if (!GroupName.IsNone() && GroupName != NAME_Groups)
		{
			StatGroupNames.Add(GroupName);
		}
	}

	void SendData()
	{
		if (NameAndInfos.Num() > 0)
		{
			// Should only do this if the delegate has been registered!
			check(NewStatGroupDelegate.IsBound());
			NewStatGroupDelegate.Execute(NameAndInfos);
			ClearData();
		}
	}

	void ClearData()
	{
		NameAndInfos.Empty();
	}

	/** Delegate we fire every time new stat groups have been registered */
	DECLARE_DELEGATE_OneParam(FOnNewStatGroupRegistered, const TArray<FStatNameAndInfo>&);
	FOnNewStatGroupRegistered NewStatGroupDelegate;

	/** A set of all the stat group names which have been registered */
	TSet<FName> StatGroupNames;

private:
	FStatGroupGameThreadNotifier(){}
	~FStatGroupGameThreadNotifier()= default;

	/** A list of all the stat groups which need broadcasting */
	TArray<FStatNameAndInfo> NameAndInfos;
};

#endif
