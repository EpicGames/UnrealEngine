// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"
#include "Misc/CoreMiscDefines.h"

#ifndef UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8
#define UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8 0
#endif

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8 || UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8

#include "Async/AsyncWork.h"
#include "Containers/Array.h"
#include "Containers/ChunkedArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/PlatformCrt.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/ThreadSafeCounter.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Build.h"
#include "Misc/Compression.h"
#include "Serialization/Archive.h"
#include "Stats/Stats.h"
#include "Stats/StatsCommon.h"
#include "Stats/StatsData.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8 || UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8

#if STATS && UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8

class FAsyncStatsWrite;
struct FStatsReadFile;

/**
* Magic numbers for stats streams, this is for the first version.
*/
enum EStatMagicNoHeader : uint32
{
	MAGIC_NO_HEADER = 0x7E1B83C1,
	MAGIC_NO_HEADER_SWAPPED = 0xC1831B7E,
	NO_VERSION = 0,
};

/**
* Magic numbers for stats streams, this is for the second and later versions.
* Allows more advanced options.
*/
enum EStatMagicWithHeader : uint32
{
	MAGIC = 0x10293847,
	MAGIC_SWAPPED = 0x47382910,
	VERSION_2 = 2,
	VERSION_3 = 3,

	/**
	*	Added support for compressing the stats data, now each frame is compressed.
	*	!!CAUTION!! Deprecated.
	*/
	VERSION_4 = 4,
	HAS_COMPRESSED_DATA_VER = VERSION_4,

	/**
	*	New low-level raw stats with memory profiler functionality.
	*  !!CAUTION!! Not backward compatible with version 4.
	*/
	VERSION_5 = 5,

	/**
	*	Added realloc message to avoid sending alloc and free
	*	Should also reduce the amount of all messages
	*  !!CAUTION!! Not backward compatible with version 5.
	*/
	VERSION_6 = 6,

	/** Latest version. */
	VERSION_LATEST = VERSION_6,
};

struct EStatsFileConstants
{
	enum
	{
		/**
		 *	Maximum size of the data that can be compressed.
		 */
		MAX_COMPRESSED_SIZE = 1024 * 1024,

		/** Header reserved for the compression internals. */
		DUMMY_HEADER_SIZE = 1024,

		/** Indicates the end of the compressed data. */
		END_OF_COMPRESSED_DATA = 0xE0F0DA4A,

		/** Indicates that the compression is disabled for the data. */
		NO_COMPRESSION = 0,
	};
};

/*-----------------------------------------------------------------------------
	FCompressedStatsData
-----------------------------------------------------------------------------*/

/** Helper struct used to operate on the compressed data. */
struct FCompressedStatsData
{
	/**
	 * Initialization constructor 
	 *	
	 * @param SrcData - uncompressed data if saving, compressed data if loading
	 * @param DestData - compressed data if saving, uncompressed data if loading
	 *
	 */
	FCompressedStatsData( TArray<uint8>& InSrcData, TArray<uint8>& InDestData )
		: SrcData( InSrcData )
		, DestData( InDestData )
		, bEndOfCompressedData( false )
	{}

	/**
	 * Writes a special data to mark the end of the compressed data.
	 */
	static void WriteEndOfCompressedData( FArchive& Writer )
	{
		int32 Marker = EStatsFileConstants::END_OF_COMPRESSED_DATA;
		check( Writer.IsSaving() );
		Writer << Marker << Marker;
	}

protected:
	/** Serialization operator. */
	friend FArchive& operator << (FArchive& Ar, FCompressedStatsData& Data)
	{
		if( Ar.IsSaving() )
		{
			Data.WriteCompressed( Ar );
		}
		else if( Ar.IsLoading() )
		{
			Data.ReadCompressed( Ar );
		}
		else
		{
			check( 0 );
		}
		return Ar;
	}

	/** Compress the data and writes to the archive. */
	void WriteCompressed( FArchive& Writer )
	{
		int32 UncompressedSize = SrcData.Num();
		if( UncompressedSize > EStatsFileConstants::MAX_COMPRESSED_SIZE - EStatsFileConstants::DUMMY_HEADER_SIZE )
		{
			int32 DisabledCompressionSize = EStatsFileConstants::NO_COMPRESSION;
			Writer << DisabledCompressionSize << UncompressedSize;
			Writer.Serialize( SrcData.GetData(), UncompressedSize );
		}
		else
		{
			DestData.Reserve( EStatsFileConstants::MAX_COMPRESSED_SIZE );
			int32 CompressedSize = (int32)DestData.GetAllocatedSize();

			const bool bResult = FCompression::CompressMemory( NAME_Zlib, DestData.GetData(), CompressedSize, SrcData.GetData(), UncompressedSize );
			check( bResult );
			Writer << CompressedSize << UncompressedSize;
			Writer.Serialize( DestData.GetData(), CompressedSize );
		}
	}

	/** Reads the data and decompresses it. */
	void ReadCompressed( FArchive& Reader )
	{
		int32 CompressedSize = 0;
		int32 UncompressedSize = 0;
		Reader << CompressedSize << UncompressedSize;

		if( CompressedSize == EStatsFileConstants::END_OF_COMPRESSED_DATA && UncompressedSize == EStatsFileConstants::END_OF_COMPRESSED_DATA )
		{
			bEndOfCompressedData = true;
		}
		// This chunk is not compressed.
		else if( CompressedSize == 0 )
		{
			DestData.Reset( UncompressedSize );
			DestData.AddUninitialized( UncompressedSize );
			Reader.Serialize( DestData.GetData(), UncompressedSize );
		}
		else
		{
			SrcData.Reset( CompressedSize );
			DestData.Reset( UncompressedSize );
			SrcData.AddUninitialized( CompressedSize );
			DestData.AddUninitialized( UncompressedSize );

			Reader.Serialize( SrcData.GetData(), CompressedSize );
			const bool bResult = FCompression::UncompressMemory(NAME_Zlib, DestData.GetData(), UncompressedSize, SrcData.GetData(), CompressedSize );
			check( bResult );
		}
	}

public:
	/**
	 * @return true if we reached the end of the compressed data.
	 */
	const bool HasReachedEndOfCompressedData() const
	{
		return bEndOfCompressedData;
	}
	
protected:
	/** Reference to the source data. */
	TArray<uint8>& SrcData;

	/** Reference to the destination data. */
	TArray<uint8>& DestData;

	/** Set to true if we reached the end of the compressed data. */
	bool bEndOfCompressedData;
};

/*-----------------------------------------------------------------------------
	Stats file writing functionality
-----------------------------------------------------------------------------*/

/** Header for a stats file. */
struct FStatsStreamHeader
{
	/** Default constructor. */
	FStatsStreamHeader()
		: Version( 0 )
		, FrameTableOffset( 0 )
		, FNameTableOffset( 0 )
		, NumFNames( 0 )
		, MetadataMessagesOffset( 0 )
		, NumMetadataMessages( 0 )
		, bRawStatsFile( false )
	{}

	/**
	 *	Version number to detect version mismatches
	 *	1 - Initial version for supporting raw ue4 stats files
	 */
	uint32	Version;

	/** Platform that this file was captured on. */
	FString	PlatformName;

	/** 
	 *  Offset in the file for the frame table. Serialized as TArray<int64>
	 *  For raw stats contains only one element which indicates the begin of the data. */
	uint64	FrameTableOffset;

	/** Offset in the file for the FName table. Serialized with WriteFName/ReadFName. */
	uint64	FNameTableOffset;

	/** Number of the FNames. */
	uint64	NumFNames;

	/** Offset in the file for the metadata messages. Serialized with WriteMessage/ReadMessage. */
	uint64	MetadataMessagesOffset;

	/** Number of the metadata messages. */
	uint64	NumMetadataMessages;

	/** Whether this stats file uses raw data, required for thread view/memory profiling/advanced profiling. */
	bool bRawStatsFile;

	/** Whether this stats file has all names stored at the end of file. */
	bool IsFinalized() const
	{
		return NumMetadataMessages > 0 && MetadataMessagesOffset > 0 && FrameTableOffset > 0;
	}

	/** Whether this stats file contains compressed data and needs the special serializer to read that data. */
	bool HasCompressedData() const
	{
		return Version >= EStatMagicWithHeader::HAS_COMPRESSED_DATA_VER;
	}

	/** Serialization operator. */
	friend FArchive& operator << (FArchive& Ar, FStatsStreamHeader& Header)
	{
		Ar << Header.Version;

		if( Ar.IsSaving() )
		{
			Header.PlatformName.SerializeAsANSICharArray( Ar, 255 );
		}
		else if( Ar.IsLoading() )
		{
			Ar << Header.PlatformName;
		}

		Ar << Header.FrameTableOffset;

		Ar << Header.FNameTableOffset
			<< Header.NumFNames;

		Ar << Header.MetadataMessagesOffset
			<< Header.NumMetadataMessages;

		Ar << Header.bRawStatsFile;

		return Ar;
	}
};

/*-----------------------------------------------------------------------------
	FStatsFrameInfo
-----------------------------------------------------------------------------*/

/**
 * Contains basic information about one frame of the stats.
 * This data is used to generate ultra-fast preview of the stats history without the requirement of reading the whole file.
 */
struct FStatsFrameInfo
{
	/** Empty constructor. */
	FStatsFrameInfo()
		: FrameFileOffset(0)
	{}

	/** Initialization constructor. */
	FStatsFrameInfo( int64 InFrameFileOffset )
		: FrameFileOffset(InFrameFileOffset)
	{}

	/** Initialization constructor. */
	FStatsFrameInfo( int64 InFrameFileOffset, TMap<uint32, int64>& InThreadCycles )
		: FrameFileOffset(InFrameFileOffset)
		, ThreadCycles(InThreadCycles)
	{}

	/** Serialization operator. */
	friend FArchive& operator << (FArchive& Ar, FStatsFrameInfo& Data)
	{
		Ar << Data.ThreadCycles << Data.FrameFileOffset;
		return Ar;
	}

	/** Frame offset in the stats file. */
	int64 FrameFileOffset;

	/** Thread cycles for this frame. */
	TMap<uint32, int64> ThreadCycles;
};

/*-----------------------------------------------------------------------------
	FStatsWriteStream
-----------------------------------------------------------------------------*/

/** Struct used to send a stream of stat messages. */
struct FStatsWriteStream
{
protected:
	/** Writes metadata messages into the stream. */
	CORE_API void WriteMetadata( FArchive& Ar );

	/** Writes condensed messages into the stream. */
	CORE_API void WriteCondensedMessages( FArchive& Ar, int64 TargetFrame );

	/** Sends an FName, and the string it represents if we have not sent that string before. **/
	inline void WriteFName( FArchive& Ar, FStatNameAndInfo NameAndInfo )
	{
		FName RawName = NameAndInfo.GetRawName();
		bool bSendFName = !FNamesSent.Contains( RawName.GetComparisonIndex() );

		int32 ComparisonInt = RawName.GetComparisonIndex().ToUnstableInt();
		Ar << ComparisonInt;

		int32 Number = NameAndInfo.GetRawNumber();
		if (bSendFName)
		{
			FNamesSent.Add( RawName.GetComparisonIndex() );
			Number |= EStatMetaFlags::SendingFName << (EStatMetaFlags::Shift + EStatAllFields::StartShift);
		}
		Ar << Number;
		if (bSendFName)
		{
			FString Name = RawName.ToString();
			Ar << Name;
		}
	}

	/** Write a stat message. **/
	inline void WriteMessage( FArchive& Ar, FStatMessage const& Item )
	{
		WriteFName( Ar, Item.NameAndInfo );
		switch (Item.NameAndInfo.GetField<EStatDataType>())
		{
			case EStatDataType::ST_int64:
			{
				int64 Payload = Item.GetValue_int64();
				Ar << Payload;
				break;
			}

			case EStatDataType::ST_double:
			{
				double Payload = Item.GetValue_double();
				Ar << Payload;
				break;
			}

			case EStatDataType::ST_FName:
			{
				WriteFName( Ar, FStatNameAndInfo( Item.GetValue_FName(), false ) );
				break;
			}

			case EStatDataType::ST_Ptr:
			{
				uint64 Payload = Item.GetValue_Ptr();
				Ar << Payload;
				break;
			}
		}
	}

	/** Set of names already sent. */
	TSet<FNameEntryId> FNamesSent;

	/** Data to write. */
	TArray<uint8> OutData;
};

/*-----------------------------------------------------------------------------
	IStatsWriteFile
-----------------------------------------------------------------------------*/

/** Interface for writing stats data. Can be only used in the stats thread. */
struct IStatsWriteFile : public FStatsWriteStream
{
	friend class FAsyncStatsWrite;

protected:
	/** Stats file archive. */
	FArchive* File;

	/** Filename of the archive that we are writing to. */
	FString ArchiveFilename;

	/** Stats stream header. */
	FStatsStreamHeader Header;

	/** Async task used to offload saving the capture data. */
	FAsyncTask<FAsyncStatsWrite>* AsyncTask;

	/** Buffer to store the compressed data, used by the FAsyncStatsWrite. */
	TArray<uint8> CompressedData;

	/**
	 *  Array of stats frames info already captured.
	 *  !!CAUTION!!
	 *  Only modified in the async write thread through FinalizeSavingData method.
	 */
	TArray<FStatsFrameInfo> FramesInfo;

	/** NewFrame delegate handle  */
	FDelegateHandle DataDelegateHandle;

	/** The size of the stats file, sets by the async writing thread. */
	int64 FileSize;

	/** Start time of the 'stat startfile', in seconds. */
	double StartTime;

protected:
	/** Default constructor. */
	CORE_API IStatsWriteFile();

public:
	/** Destructor. */
	virtual ~IStatsWriteFile() = default;

	/** Creates a file writer and registers for the data delegate. */
	CORE_API void Start( const FString& InFilename );

	/** Finalizes writing the stats data and unregisters the data delegate. */
	CORE_API void Stop();

	/**
	 * @return the stats file metadata description like the current size and the duration of the stats session.
	 */
	CORE_API FText GetFileMetaDesc() const;

protected:
	/** Sets the data delegate used to receive a stats data. */
	virtual void SetDataDelegate( bool bSet ) = 0;

	/** Finalization code called after the compressed data has been saved. */
	virtual void FinalizeSavingData( int64 FrameFileOffset )
	{};

	bool IsValid() const
	{
		return !!File;
	}

	/** Writes magic value, dummy header and initial metadata. */
	CORE_API void WriteHeader();

protected:
	/**	Finalizes writing to the file. */
	CORE_API void Finalize();

	/** Waits on the async send task to complete. */
	CORE_API void WaitTask();

	/** Sends the data to the file via async task. */
	CORE_API void SendTask();
};

/** Helper struct used to write regular stats to the file. */
struct FStatsWriteFile : public IStatsWriteFile
{
	friend class FAsyncStatsWrite;

protected:
	/** Thread cycles for the last frame. */
	TMap<uint32, int64> ThreadCycles;

public:
	/** Default constructor, set bRawStatsFile to false. */
	FStatsWriteFile()
	{
		Header.bRawStatsFile = false;
	}

protected:
	CORE_API virtual void SetDataDelegate( bool bSet ) override;

	CORE_API virtual void FinalizeSavingData( int64 FrameFileOffset ) override;

	/**
	 *	Grabs a frame from the local FStatsThreadState and adds it to the output.
	 *	Called from the stats thread, but the data is saved using the the FAsyncStatsWrite. 
	 */
	CORE_API void WriteFrame( int64 TargetFrame );
};

/** Helper struct used to write raw stats to the file. */
struct FRawStatsWriteFile : public IStatsWriteFile
{
	bool bWrittenOffsetToData;

public:
	/** Default constructor, set bRawStatsFile to true. */
	FRawStatsWriteFile()
		: bWrittenOffsetToData(false)
	{
		Header.bRawStatsFile = true;
	}

protected:
	CORE_API virtual void SetDataDelegate( bool bSet ) override;

	CORE_API void WriteRawStatPacket( const FStatPacket* StatPacket );

	/** Write a stat packed into the specified archive. */
	CORE_API void WriteStatPacket( FArchive& Ar, FStatPacket& StatPacket );
};

/*-----------------------------------------------------------------------------
	Stats file reading functionality
-----------------------------------------------------------------------------*/

/** Tracks stat state and history for loaded stats. */
class FStatsLoadedState : public FStatsThreadState
{
	friend struct FStatsReadFile;

public:
	/** Default constructor. */
	FStatsLoadedState()
		: MaxFrameSeen( 0 )
		, MinFrameSeen( -1 )
	{}

	/** Method to update the internal metadata. Metadata messages are removed from the array. */
	CORE_API void ProcessMetaDataAndLeaveDataOnly( TArray<FStatMessage>& CondensedMessages );

	/**
	* Method to place the data into the history,
	* maintains the history based on the requested number of frames to keep in the history.
	* The condensed messages are emplaced in the condensed history.
	*/
	CORE_API void AddFrameFromCondensedMessages( TArray<FStatMessage>& CondensedMessages );

	/** Return the oldest frame of data we have. **/
	CORE_API int64 GetOldestValidFrame() const;

	/** Return the newest frame of data we have. **/
	CORE_API int64 GetLatestValidFrame() const;

protected:
	/** Internal method to scan the messages to find the current game/render thread frame. */
	CORE_API void AdvanceFrameForLoad( TArray<FStatMessage>& CondensedMessages );

	/** Largest frame seen. **/
	int64 MaxFrameSeen;

	/** First frame seen. **/
	int64 MinFrameSeen;
};

/**
 * Class for maintaining state of receiving a stream of stat messages
 */
struct FStatsReadStream
{
public:
	/** Stats stream header. */
	FStatsStreamHeader Header;

	/** FNames have a different index on each machine, so we translate via this map. **/
	TMap<int32, FNameEntryId> FNamesIndexMap;

	/** Array of stats frame info. Empty for the raw stats. */
	TArray<FStatsFrameInfo> FramesInfo;

	/** Reads a stats stream header, returns true if the header is valid and we can continue reading. */
	bool ReadHeader( FArchive& Ar )
	{
		bool bStatWithHeader = false;

		uint32 Magic = 0;
		Ar << Magic;
		if( Magic == EStatMagicNoHeader::MAGIC_NO_HEADER )
		{

		}
		else if( Magic == EStatMagicNoHeader::MAGIC_NO_HEADER_SWAPPED )
		{
			Ar.SetByteSwapping( true );
		}
		else if( Magic == EStatMagicWithHeader::MAGIC )
		{
			bStatWithHeader = true;
		}
		else if( Magic == EStatMagicWithHeader::MAGIC_SWAPPED )
		{
			bStatWithHeader = true;
			Ar.SetByteSwapping( true );
		}
		else
		{
			return false;
		}

		// We detected a header for a stats file, read it.
		if( bStatWithHeader )
		{
			Ar << Header;
		}

		return true;
	}

	/** Reads a stat packed from the specified archive. Only for raw stats files. */
	void ReadStatPacket( FArchive& Ar, FStatPacket& StatPacked )
	{
		Ar << StatPacked.Frame;
		Ar << StatPacked.ThreadId;
		int32 MyThreadType = 0;
		Ar << MyThreadType;
		StatPacked.ThreadType = (EThreadType::Type)MyThreadType;

		Ar << StatPacked.bBrokenCallstacks;
		// We must handle stat messages in a different way.
		int32 NumMessages = 0;
		Ar << NumMessages;
		StatPacked.StatMessages.Reserve( NumMessages );
		for( int32 MessageIndex = 0; MessageIndex < NumMessages; ++MessageIndex )
		{
			new(StatPacked.StatMessages) FStatMessage( ReadMessage( Ar, true ) );
		}
	}

	/** Read and translate or create an FName. **/
	inline FStatNameAndInfo ReadFName( FArchive& Ar, bool bHasFNameMap )
	{
		// If we read the whole FNames translation map, we don't want to add the FName again.
		// This is a bit tricky, even if we have the FName translation map, we still need to read the FString.
		// CAUTION!! This is considered to be thread safe in this case.
		int32 Index = 0;
		Ar << Index;
		int32 Number = 0;
		Ar << Number;
		FName TheFName;
		
		if( !bHasFNameMap )
		{
			if( Number & (EStatMetaFlags::SendingFName << (EStatMetaFlags::Shift + EStatAllFields::StartShift)) )
			{
				FString Name;
				Ar << Name;

				TheFName = FName( *Name );
				FNamesIndexMap.Add( Index, TheFName.GetComparisonIndex() );
				Number &= ~(EStatMetaFlags::SendingFName << (EStatMetaFlags::Shift + EStatAllFields::StartShift));
			}
			else
			{
				if( FNamesIndexMap.Contains( Index ) )
				{
					const FNameEntryId MyIndex = FNamesIndexMap.FindChecked( Index );
					TheFName = FName( MyIndex, MyIndex, 0 );
				}
				else
				{
					TheFName = FName( "Unknown FName" );
					Number = 0;
					UE_LOGF( LogTemp, Warning, "Missing FName Indexed: %d, %d", Index, Number );
				}
			}
		}
		else
		{
			if( Number & (EStatMetaFlags::SendingFName << (EStatMetaFlags::Shift + EStatAllFields::StartShift)) )
			{
				FString Name;
				Ar << Name;
				Number &= ~(EStatMetaFlags::SendingFName << (EStatMetaFlags::Shift + EStatAllFields::StartShift));
			}
			if( FNamesIndexMap.Contains( Index ) )
			{
				const FNameEntryId MyIndex = FNamesIndexMap.FindChecked( Index );
				TheFName = FName( MyIndex, MyIndex, 0 );
			}
			else
			{
				TheFName = FName( "Unknown FName" );
				Number = 0;
				UE_LOGF( LogTemp, Warning, "Missing FName Indexed: %d, %d", Index, Number );
			}
		}

		FStatNameAndInfo Result( TheFName, false );
		Result.SetNumberDirect( Number );
		return Result;
	}

	/** Read a stat message. **/
	inline FStatMessage ReadMessage( FArchive& Ar, bool bHasFNameMap = false )
	{
		FStatMessage Result( ReadFName( Ar, bHasFNameMap ) );
		Result.Clear();
		switch( Result.NameAndInfo.GetField<EStatDataType>() )
		{
			case EStatDataType::ST_int64:
			{
				int64 Payload = 0;
				Ar << Payload;
				Result.GetValue_int64() = Payload;
				break;
			}
				
			case EStatDataType::ST_double:
			{
				double Payload = 0;
				Ar << Payload;
				Result.GetValue_double() = Payload;
				break;
			}
				
			case EStatDataType::ST_FName:
			{
				FStatNameAndInfo Payload( ReadFName( Ar, bHasFNameMap ) );
				Result.GetValue_FMinimalName() = NameToMinimalName( Payload.GetRawName() );
				break;
			}

			case EStatDataType::ST_Ptr:
			{
				uint64 Payload = 0;
				Ar << Payload;
				Result.GetValue_Ptr() = Payload;
				break;
			}
		}
		return Result;
	}

	/**
	 *	Reads stats frames info from the specified archive, only valid for finalized stats files.
	 *	Allows unordered file access and whole data mini-view.
	 */
	void ReadFramesOffsets( FArchive& Ar )
	{
		Ar.Seek( Header.FrameTableOffset );
		Ar << FramesInfo;
	}

	/**
	 *	Reads FNames and metadata messages from the specified archive, only valid for finalized stats files.
	 *	Allow unordered file access.
	 */
	void ReadFNamesAndMetadataMessages(FArchive& Ar, TArray64<FStatMessage>& out_MetadataMessages)
	{
		// Read FNames.
		Ar.Seek(Header.FNameTableOffset);
		out_MetadataMessages.Reserve(Header.NumFNames);
		for (uint64 Index = 0; Index < Header.NumFNames; Index++)
		{
			ReadFName(Ar, false);
		}

		// Read metadata messages.
		Ar.Seek(Header.MetadataMessagesOffset);
		out_MetadataMessages.Reserve(Header.NumMetadataMessages);
		for (uint64 Index = 0; Index < Header.NumMetadataMessages; Index++)
		{
			out_MetadataMessages.Add(ReadMessage(Ar, false));
		}
	}
};

/** Raw stats information. */
struct FRawStatsFileInfo
{
	/** Default constructor. */
	FRawStatsFileInfo()
		: TotalPacketsSize( 0 )
		, TotalStatMessagesNum( 0 )
		, MaximumPacketSize( 0 )
		, TotalPacketsNum( 0 )
	{}

	/** Size of all packets. */
	int32 TotalPacketsSize;

	/** Number of all stat messages. */
	int32 TotalStatMessagesNum;

	/** Maximum packet size. */
	int32 MaximumPacketSize;

	/** Number of all packets. */
	int32 TotalPacketsNum;
};

/** Enumerates stats processing stages. */
enum class EStatsProcessingStage : int32
{
	/** Started loading. */
	SPS_Started = 0,

	/** Read and combine packets. */
	SPS_ReadStats,

	/** Pre process stats, prepare. */
	SPS_PreProcessStats,

	/** Process combine history. */
	SPS_ProcessStats,

	/** Post process stats, finalize. */
	SPS_PostProcessStats,

	/** Finished processing. */
	SPS_Finished,

	/** Stopped processing. */
	SPS_Stopped,

	/** Last stage, at this moment all data is read and processed or process has been stopped, we can now remove the instance. */
	SPS_Invalid,

};

/**
 *	Helper class used to read and process stats file on the async task.
 */
class FAsyncStatsFile
{
	/** Pointer to FStatsReadFile to call for async work. */
	FStatsReadFile* Owner;

public:
	/** Initialization constructor. */
	FAsyncStatsFile( FStatsReadFile* InOwner );

	/** Call DoWork on the parent */
	void DoWork();

	TStatId GetStatId() const
	{
		return TStatId();
	}

	/** This task is abandonable. */
	bool CanAbandon()
	{
		return true;
	}

	/** Abandon this task. */
	void Abandon();
};

/*-----------------------------------------------------------------------------
	Stats stack helpers
-----------------------------------------------------------------------------*/

/** Holds stats stack state, used to preserve continuity when the game frame has changed. For raw stats. */
struct FStackState
{
	/** Default constructor. */
	FStackState()
		: bIsBrokenCallstack( false )
	{}

	/** Call stack. */
	TArray<FName> Stack;

	/** Current function name. */
	FName Current;

	/** Whether this callstack is marked as broken due to mismatched start and end scope cycles. */
	bool bIsBrokenCallstack;
};

/*-----------------------------------------------------------------------------
	FStatsReadFile
-----------------------------------------------------------------------------*/

template<typename T>
struct FStatsReader
{
	/** Creates a new reader for raw/regular stats file based on the T. Will be nullptr for invalid files. */	
	static T* Create( const TCHAR* Filename )
	{
		T* StatsReadFile = new T( Filename );
		const bool bValid = StatsReadFile->PrepareLoading();
		if (!bValid)
		{
			delete StatsReadFile;
		}
		return bValid ? StatsReadFile : nullptr;
	}
};

/** Struct used to read from uestats/uestatsraw files, initializes all metadata and starts a process of reading the file asynchronously. */
struct FStatsReadFile
{
	friend class FAsyncStatsFile;
	friend struct FStatsReader<FStatsReadFile>;

	/** Number of seconds between updating the current stage. */
	static CORE_API const double NumSecondsBetweenUpdates;

public:
	/** Reads and processes the file on the current thread. This is a blocking operation. */
	CORE_API void ReadAndProcessSynchronously();

	/** Reads and processes the file using the async tasks on the pool thread. The read data is sent to the game thread using the task graph. This is a non-blocking operation. */
	CORE_API void ReadAndProcessAsynchronously();

	/** Sets the number of frame to be stored for future use. */
	CORE_API void SetHistoryFrames( int32 InHistoryFrames );

	/**
	 * @return number of frames in the file.
	 * For regular stat file valid after creating a reader.
	 * For raw stat file reading whole file.
	 */
	int32 GetNumFrames() const
	{
		return NumFrames;
	}

protected:
	/** Initialization constructor. */
	CORE_API FStatsReadFile( const TCHAR* InFilename, bool bInRawStatsFile );

public:
	/** Destructor. */
	CORE_API virtual ~FStatsReadFile();

protected:
	/**
	 * Prepares file to be loaded, makes sanity checks, reads and initializes metadata
	 * @return true, if the process was completed successfully
	 */
	CORE_API bool PrepareLoading();

	/** Reads stats from the file. */
	CORE_API void ReadStats();

	/** Reads raw stats. */
	CORE_API void ReadRawStats();

	/** Reads regular stats. */
	CORE_API void ReadRegularStats();

	/** Called before started processing. */
	CORE_API virtual void PreProcessStats();

	/** Processes combined history using the internal functionality and provided overloaded Process*Operation methods. */
	CORE_API void ProcessStats();

	/** Called every each frame has been read from the file. */
	virtual void ReadStatsFrame( const TArray<FStatMessage>& CondensedMessages, const int64 Frame )
	{}

	/** Called after finished processing combined history. */
	CORE_API virtual void PostProcessStats();

	/** Processes special message for advancing the stats frame from the game thread. */
	virtual void ProcessAdvanceFrameEventGameThreadOperation( const FStatMessage& Message, const FStackState& StackState )
	{}

	/** Processes special message for advancing the stats frame from the render thread. */
	virtual void ProcessAdvanceFrameEventRenderThreadOperation( const FStatMessage& Message, const FStackState& StackState )
	{}

	/** ProcessesIndicates begin of the cycle scope. */
	virtual void ProcessCycleScopeStartOperation( const FStatMessage& Message, const FStackState& StackState )
	{}

	/** Indicates end of the cycle scope. */
	virtual void ProcessCycleScopeEndOperation( const FStatMessage& Message, const FStackState& StackState )
	{}

	/** Processes special message marker used determine that we encountered a special data in the stat file. */
	virtual void ProcessSpecialMessageMarkerOperation( const FStatMessage& Message, const FStackState& StackState )
	{}

	/** Sets a new processing stage for this file. */
	void SetProcessingStage( EStatsProcessingStage NewStage )
	{
		if (GetProcessingStage() != NewStage)
		{
			ProcessingStage.Set( int32( NewStage ) );
			StageProgress.Set( 0 );
		}
	}

public:
	/**
	 * @return current processing stage.
	 */
	const EStatsProcessingStage GetProcessingStage() const
	{
		return EStatsProcessingStage( ProcessingStage.GetValue() );
	}

	/**
	 * @return true, if a user stopped the processing, probably using the Cancel button in the UI.
	 */
	const bool IsProcessingStopped() const
	{
		return GetProcessingStage() == EStatsProcessingStage::SPS_Stopped;
	}

	/**
	 * @return current processing stage as string key.
	 */
	FString GetProcessingStageAsString() const
	{
		const EStatsProcessingStage Stage = GetProcessingStage();
		FString Result;
		if (Stage== EStatsProcessingStage::SPS_Started)
		{
			Result = TEXT( "SPS_Started" );
		}
		else if (Stage == EStatsProcessingStage::SPS_ReadStats)
		{
			Result = TEXT( "SPS_ReadStats" );
		}
		else if (Stage == EStatsProcessingStage::SPS_PreProcessStats)
		{
			Result = TEXT( "SPS_PreProcessStats" );
		}
		else if (Stage == EStatsProcessingStage::SPS_ProcessStats)
		{
			Result = TEXT( "SPS_ProcessStats" );
		}
		else if (Stage == EStatsProcessingStage::SPS_PostProcessStats)
		{
			Result = TEXT( "SPS_PostProcessStats" );
		}
		else if (Stage == EStatsProcessingStage::SPS_Finished)
		{
			Result = TEXT( "SPS_Finished" );
		}
		else if (Stage == EStatsProcessingStage::SPS_Stopped)
		{
			Result = TEXT( "SPS_Stopped" );
		}
		else if (Stage == EStatsProcessingStage::SPS_Invalid)
		{
			Result = TEXT( "SPS_Invalid" );
		}

		return Result;
	}

	/**
	 * @return current stage progress as percentage, between 0 and 100.
	 */
	int32 GetStageProgress() const
	{
		return StageProgress.GetValue();
	}

	/**	 
	 * @return true, if any asynchronous operation is being processed
	 */
	bool IsBusy()
	{
		return AsyncWork != nullptr ? !AsyncWork->IsDone() : false;
	}

	/** Requests stopping the processing of the data. May take a few seconds to finish. */
	void RequestStop()
	{
		bShouldStopProcessing = true;
	}

protected:
	/**
	 * Updates read stage progress periodically, does debug logging if enabled.
	 */
	CORE_API void UpdateReadStageProgress();

	/** Dumps combined history stats. Only for raw stats. */
	CORE_API void UpdateCombinedHistoryStats();

	/**
	 * Updates process stage progress periodically, does debug logging if enabled.
	 */
	CORE_API void UpdateProcessStageProgress( const int32 CurrentStatMessageIndex, const int32 FrameIndex, const int32 PacketIndex );

protected:
	/** Current state of the stats. Mostly for metadata. */
	FStatsLoadedState State;

	/** Reading stream. */
	FStatsReadStream Stream;

	/** Reference to the stream's header. */
	FStatsStreamHeader& Header;

	/** File reader. */
	FArchive* Reader;

	/** Async task. */
	FAsyncTask<FAsyncStatsFile>* AsyncWork;

	/** Basic information about the stats file. */
	FRawStatsFileInfo FileInfo;

	/** Combined history for raw packets, indexed by a frame number. */
	TMap<int32, FStatPacketArray> CombinedHistory;

	/** Frames computed from the combined history. */
	TArray<int32> Frames;

	/** All raw names that contains a path to an UObject. */
	TSet<FName> UObjectRawNames;

	/** Current stage of processing. */
	FThreadSafeCounter ProcessingStage;

	/** Percentage progress of the current stage. */
	FThreadSafeCounter StageProgress;

	/** If true, we should break the processing loop. */
	FThreadSafeBool bShouldStopProcessing;

	/** Time of the last stage update. */
	double LastUpdateTime;

	/** Filename. */
	const FString Filename;

	/** Number of frames. */
	int32 NumFrames;

	/** Whether this stats file uses raw data. */
	const bool bRawStatsFile;
};

/*-----------------------------------------------------------------------------
	Commands functionality
-----------------------------------------------------------------------------*/

/** Implements 'Stat Start/StopFile' functionality. */
struct FCommandStatsFile
{
	/** Access to the singleton. */
	static CORE_API FCommandStatsFile& Get();

	/** Default constructor. */
	FCommandStatsFile()
		: FirstFrame(-1)
		, CurrentStatsFile(nullptr)
	{}

	/** Stat StartFile. */
	void Start( const FString& Filename );

	/** Stat StartFileRaw. */
	void StartRaw( const FString& Filename );

	/** Stat StopFile. */
	void Stop();

	/** Test file by loading the last capture. */
	void TestLastSaved();

	/**
	 * @return true, if any stats write command is active.
	 */
	bool IsStatFileActive() const
	{
		return StatFileActiveCounter.GetValue() > 0;
	}

	/**
	* @return the stats file metadata description like the current size and the duration of the stats session.
	*/
	FText GetFileMetaDesc() const
	{
		if (IsStatFileActive() && CurrentStatsFile != nullptr)
		{
			return CurrentStatsFile->GetFileMetaDesc();
		}
		return FText();
	}

	/** Filename of the last saved stats file. */
	FString LastFileSaved;

protected:
	/** First frame. */
	int64 FirstFrame;

	/** Whether the 'stat startfile' command is active, can be accessed by other thread. */
	FThreadSafeCounter StatFileActiveCounter;

	/** Stats file that is currently being saved. */
	IStatsWriteFile* CurrentStatsFile;
};


#endif // STATS && UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8
