// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformTime.h"
#include "HAL/PreprocessorHelpers.h"
#include "Logging/MessageLog.h"
#include "Misc/Build.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FMessageLog;
class FStatsHierarchical;

/**
 *	An element in a profiling / stats tree
 *
 *	The FStatsTreeElement represents a single node in a profiling tree. Each element contains
 *	a description including its name, path (invocation path / call-stack) and its duration in 
 *  system cycles.
 */
class FStatsTreeElement
{
public:

	// Default constructor
	CORE_API FStatsTreeElement();

	// Returns the name of this element
	CORE_API FName GetFName() const;

	// Returns the name of this element as a FString
	CORE_API FString GetName() const;

	/* 
		Returns the invocation path of this element.
		The path is a period-separated string of all of the nested
		profiling scopes.
		For example: main.MyClass::Method.Algo::Sort
	*/
	CORE_API FString GetPath() const;

	// Returns the number of invocations collected into this element
	// @param bInclusive If true the invocations will contain the invocations used by child elements
	CORE_API uint32 Num(bool bInclusive = false) const;

	// Returns the total number of cycles recorded
	// @param bInclusive If true the cycles will contain the cycles used by child elements
	CORE_API uint32 TotalCycles(bool bInclusive = true) const;

	// Returns the number of maximum cycles for this element (and children)
	CORE_API uint32 MaxCycles(bool bInclusive = true) const;

	// Returns the total number of seconds recorded
	// @param bInclusive If true the time will contain the cycles used by child elements
	CORE_API double TotalSeconds(bool bInclusive = true) const;

	// Returns the average number of seconds recorded (total / num)
	// @param bInclusive If true the time will contain the cycles used by child elements
	CORE_API double AverageSeconds(bool bInclusive = true) const;

	// Returns the contribution between 0.0 and 1.0 within the parent element.
	// 1.0 means that 100% of the time of the parent element is spent in this child.
	// @param bAgainstMaximum If true the ratio is expressed against the largest time in the tree
	// @param bInclusive If true the time will contain the cycles used by child elements
	CORE_API double Contribution(bool bAgainstMaximum = false, bool bInclusive = true) const;

	// Returns all child elements
	CORE_API const TArray<TSharedPtr<FStatsTreeElement>>& GetChildren() const;

protected:

	FName Name;
	FString Path;
	uint32 Invocations;
	uint32 Cycles;

	// derived data (computed by UpdatePoseMeasurement)
	uint32 CyclesOfChildren;
	double RatioAgainstTotalInclusive;
	double RatioAgainstTotalExclusive;
	double RatioAgainstMaximumInclusive;
	double RatioAgainstMaximumExclusive;

	// children of the tree
	TArray<TSharedPtr<FStatsTreeElement>> Children;

	// returns the child or nullptr based on a given path
	CORE_API FStatsTreeElement* FindChild(const FString& InPath);

	CORE_API void UpdatePostMeasurement(double InCyclesPerTimerToRemove = 0);

	friend class FStatsHierarchical;
	friend class FStatsHierarchicalClient;
};

#if STATS

// Used to declare a hierarchical counter. The information about all of the counters can 
// be retrieved by FStatsHierarhical::GetLastMeasurements.
// Note: You need to call FStatsHierarhical::BeginMeasurements && FStatsHierarhical::EndMeasurements
// for this to have an effect.
#define DECLARE_SCOPE_HIERARCHICAL_COUNTER(CounterName) \
	FStatsHierarchical::FScope UE_JOIN(StatsHierarchicalScope, __LINE__)(#CounterName);
#define DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC() \
	FStatsHierarchical::FScope UE_JOIN(StatsHierarchicalScope, __LINE__)(__FUNCTION__);

#else

#define DECLARE_SCOPE_HIERARCHICAL_COUNTER(CounterName)
#define DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

#endif

/**
 * A helper class with static methods to perform hierarchical profiling
 * 
 * The FStatsHierarchical class / namespace can be used to record profiling data
 * in a hierarchical fashion - by nesting scopes in functions that call each other.
 * Profiling entries are recorded per invocation as raw data to ensure minimal
 * impact on the runtime performance - and then can be compacted into a tree
 * at a later time for reporting / display.
 * 
 * Users should refer to the DECLARE_SCOPE_HIERARCHICAL_COUNTER macro instead to place scopes.
 *
 * Users need to call FStatsHierarchical::BeginMeasurements() and 
 * FStatsHierarchical::EndMeasurements() to enable / disable the profiling.
 * 
 * Note: This system is not thread-safe. You want to call it from a single
 * thread only.
 */
class FStatsHierarchical
{
public:

	// Helper class to create a local scope for profiling.
	// Calls the static methods on FStatsHierarchical.
	// Users should use the DECLARE_SCOPE_HIERARCHICAL_COUNTER macro instead.
	struct FScope
	{
		FScope(const ANSICHAR * InLabel)
		{
#if STATS
			FStatsHierarchical::BeginMeasurement(InLabel);
#endif
		}

		~FScope()
		{
#if STATS
			FStatsHierarchical::EndMeasurement();
#endif
		}
	};

	// Enabled measurements / profiling
	static CORE_API void BeginMeasurements();

	// Returns true if measurements are enabled
	static CORE_API bool IsEnabled();

	/**
	 * Ends measurements / profiling and returns the compacted profiling tree.
	 * @param MeasurementsToMerge The baseline for the measurement.
	 * @param bAddUntrackedElements If true adds an element for untracked time (time not profiled) for each node in the tree.
	 */
	static CORE_API FStatsTreeElement EndMeasurements(FStatsTreeElement MeasurementsToMerge = FStatsTreeElement(), bool bAddUntrackedElements = true);

	// Returns the last recorded profiling tree
	static CORE_API FStatsTreeElement GetLastMeasurements();

	// Prints the results into a provided log
	static CORE_API void DumpMeasurements(FMessageLog& Log, bool bSortByDuration = true);

	// Returns the name to use for untracked time
	static CORE_API FName GetUntrackedTimeName();

private:

	// A single entry for profiling.
	// This data is used internally only and should not
	// be used directly.
	struct FHierarchicalStatEntry
	{
		FHierarchicalStatEntry(const ANSICHAR * InLabel, uint32 InCycles)
			:Label(InLabel)
			, Cycles(InCycles)
		{
		}

		FHierarchicalStatEntry()
			:Label(nullptr)
			, Cycles(0)
		{
		}

		const ANSICHAR * Label;
		uint32 Cycles;

		friend class FStatsHierarchical;
	};

	/**
	 * Begins a single measurement given a label.
	 * @param Label The label for this measurement (has to be != nullptr)
	 */
	static CORE_API void BeginMeasurement(const ANSICHAR * Label);

	// Ends the last measurement
	static CORE_API void EndMeasurement();

	static CORE_API bool bEnabled;
	static CORE_API TArray<FHierarchicalStatEntry> Entries;

	friend struct FStatsHierarchical::FScope;
};
