// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PlatformTime.h"
#include "HAL/PreprocessorHelpers.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Build.h"
#include "Templates/UnrealTemplate.h"

/**
* Utility class to capture time passed in seconds, adding delta time to passed
* in variable. Not useful for reentrant functions
*/
class FSimpleScopeSecondsCounter
{
public:
	/** Ctor, capturing start time. */
	FSimpleScopeSecondsCounter(double& InSeconds, bool bInEnabled = true)
		: StartTime(bInEnabled ? FPlatformTime::Seconds() : 0.0)
		, Seconds(InSeconds)
		, bEnabled(bInEnabled)
		, RecursionDepth(nullptr)
	{}
	/** Ctor, capturing start time. */
	FSimpleScopeSecondsCounter(double& InSeconds, int32* InRecursionDepth)
		: StartTime(FPlatformTime::Seconds())
		, Seconds(InSeconds)
		, bEnabled(*InRecursionDepth == 0)
		, RecursionDepth(InRecursionDepth)
	{
		(*RecursionDepth)++;
	}
	/** Dtor, updating seconds with time delta. */
	~FSimpleScopeSecondsCounter()
	{
		if (bEnabled)
		{
			Seconds += FPlatformTime::Seconds() - StartTime;
		}

		if (RecursionDepth)
		{
			(*RecursionDepth)--;
		}
	}
private:
	/** Start time, captured in ctor. */
	double StartTime;
	/** Time variable to update. */
	double& Seconds;
	/** Is the timer enabled or disabled */
	bool bEnabled;
	/** Recursion depth */
	int32* RecursionDepth;
};

/** Macro for updating a seconds counter without creating new scope. */
#define SCOPE_SECONDS_COUNTER_BASE(Seconds) \
	FSimpleScopeSecondsCounter SecondsCount_##Seconds(Seconds);

#define SCOPE_SECONDS_COUNTER_RECURSION_SAFE_BASE(Seconds) \
	static int32 SecondsCount_##Seconds##_RecursionCounter = 0; \
	FSimpleScopeSecondsCounter SecondsCount_##Seconds(Seconds, &SecondsCount_##Seconds##_RecursionCounter);

#if STATS
#define SCOPE_SECONDS_COUNTER(Seconds) SCOPE_SECONDS_COUNTER_BASE(Seconds)
#define SCOPE_SECONDS_COUNTER_RECURSION_SAFE(Seconds) SCOPE_SECONDS_COUNTER_RECURSION_SAFE_BASE(Seconds)
#else
#define SCOPE_SECONDS_COUNTER(Seconds)
#define SCOPE_SECONDS_COUNTER_RECURSION_SAFE(Seconds)
#endif 


/**
* Utility class to store a counter and a time value in seconds. Implementation will be stripped out in 
* STATS disabled builds, although it will waste a small amount of memory unless stripped by the linker.
*
* Useful when timing reentrant functions.
*/
struct FSecondsCounterData
{
#if STATS
	FSecondsCounterData()
		: Time(0.0)
		, ScopeCounter(0)
	{
	}

	double GetTime() const { return Time; }
	void ClearTime() { check(ScopeCounter == 0);  Time = 0.0; }
private:
	double Time;
	int32 ScopeCounter;

	friend class FSecondsCounterScope;
#else //STATS
public:
	static double GetTime() { return 0.0; }
	static void ClearTime() {}
#endif //STATS
};

/**
* Utility class to update a FSecondsCounterData. Does nothing in STATS disabled builds.
*/
class FSecondsCounterScope
{
#if STATS
public:
	FSecondsCounterScope(FSecondsCounterData& InData)
		: Data(InData)
		, StartTime(-1.f)
	{
		if (Data.ScopeCounter == 0)
		{
			StartTime = FPlatformTime::Seconds();
		}
		++InData.ScopeCounter;
	}

	~FSecondsCounterScope()
	{
		--Data.ScopeCounter;
		if (Data.ScopeCounter == 0)
		{
			checkf(StartTime >= 0.f, TEXT("Counter is corrupt! Thinks it started before epoch"));
			Data.Time += (FPlatformTime::Seconds() - StartTime);
		}
	}

private:
	FSecondsCounterData& Data;
	double StartTime;
#else //STATS
public:
	FSecondsCounterScope(FSecondsCounterData& InData) {}
#endif //STATS
};


/** Key contains total time, value contains total count. */
typedef TKeyValuePair<double, uint32> FTotalTimeAndCount;

/**
 *	Utility class to log time passed in seconds, adding cumulative stats to passed in variable. 
 */
struct FConditionalScopeLogTime
{
	enum EScopeLogTimeUnits
	{
		ScopeLog_DontLog,
		ScopeLog_Milliseconds,
		ScopeLog_Seconds
	};

	/**
	 * Initialization constructor.
	 *
	 * @param InName - String that will be displayed in the log
	 * @param InGlobal - Pointer to the variable that holds the cumulative stats
	 *
	 */
	CORE_API FConditionalScopeLogTime( bool bCondition, const WIDECHAR* InName, FTotalTimeAndCount* InCumulative = nullptr, EScopeLogTimeUnits InUnits = ScopeLog_Milliseconds );
	CORE_API FConditionalScopeLogTime( bool bCondition, const ANSICHAR* InName, FTotalTimeAndCount* InCumulative = nullptr, EScopeLogTimeUnits InUnits = ScopeLog_Milliseconds );

	/** Destructor. */
	CORE_API ~FConditionalScopeLogTime();

protected:
	double GetDisplayScopedTime(double InScopedTime) const;
	FString GetDisplayUnitsString() const;


	const double StartTime;
	const FString Name;
	FTotalTimeAndCount* Cumulative;
	EScopeLogTimeUnits Units;
};

/**
 *	Utility class to log time passed in seconds, adding cumulative stats to passed in variable. 
 */
struct FScopeLogTime : public FConditionalScopeLogTime
{
	FScopeLogTime(const WIDECHAR* InName, FTotalTimeAndCount* InCumulative = nullptr, EScopeLogTimeUnits InUnits = ScopeLog_Milliseconds)
		: FConditionalScopeLogTime(true, InName, InCumulative, InUnits)
	{
	}

	FScopeLogTime(const ANSICHAR* InName, FTotalTimeAndCount* InCumulative = nullptr, EScopeLogTimeUnits InUnits = ScopeLog_Milliseconds)
		: FConditionalScopeLogTime(true, InName, InCumulative, InUnits)
	{
	}
};

#define SCOPE_LOG_TIME(Name,CumulativePtr) \
	FConditionalScopeLogTime UE_JOIN(ScopeLogTime,__LINE__)(true, Name, CumulativePtr);

#define SCOPE_LOG_TIME_IN_SECONDS(Name,CumulativePtr) \
	FConditionalScopeLogTime UE_JOIN(ScopeLogTime,__LINE__)(true, Name, CumulativePtr, FScopeLogTime::ScopeLog_Seconds);

#define SCOPE_LOG_TIME_FUNC() \
	FConditionalScopeLogTime UE_JOIN(ScopeLogTime,__LINE__)(true, __FUNCTION__);

#define SCOPE_LOG_TIME_FUNC_WITH_GLOBAL(CumulativePtr) \
	FConditionalScopeLogTime UE_JOIN(ScopeLogTime,__LINE__)(true, __FUNCTION__,CumulativePtr);

#define CONDITIONAL_SCOPE_LOG_TIME(Condition, Name,CumulativePtr) \
	FConditionalScopeLogTime UE_JOIN(ScopeLogTime,__LINE__)(Condition, Name, CumulativePtr);

#define CONDITIONAL_SCOPE_LOG_TIME_IN_SECONDS(Condition, Name,CumulativePtr) \
	FConditionalScopeLogTime UE_JOIN(ScopeLogTime,__LINE__)(Condition, Name, CumulativePtr, FScopeLogTime::ScopeLog_Seconds);

#define CONDITIONAL_SCOPE_LOG_TIME_FUNC(Condition) \
	FConditionalScopeLogTime UE_JOIN(ScopeLogTime,__LINE__)(Condition, __FUNCTION__);

#define CONDITIONAL_SCOPE_LOG_TIME_FUNC_WITH_GLOBAL(Condition, CumulativePtr) \
	FConditionalScopeLogTime UE_JOIN(ScopeLogTime,__LINE__)(Condition, __FUNCTION__,CumulativePtr);
