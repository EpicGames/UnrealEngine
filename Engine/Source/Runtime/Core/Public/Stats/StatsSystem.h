// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateBase.h"
#include "Templates/Atomic.h"

namespace UE::Stats
{
	/** Helper struct that contains method available even when the stats are disabled. */
	struct FStats
	{
		/** Delegate to fire every time we need to advance the stats for the rendering thread. */
		DECLARE_DELEGATE_ThreeParams(FOnAdvanceRenderingThreadStats, bool /*bDiscardCallstack*/, int64 /*StatsFrame*/, int32 /*PrimaryDisableChangeTagStartFrame*/);

		/* Initializes any relevant stat systems, including those that exist when STATS is 0. */
		static CORE_API void Init();

		/** Advances stats for the current frame. */
		static CORE_API void AdvanceFrame(bool bDiscardCallstack, const FOnAdvanceRenderingThreadStats& AdvanceRenderingThreadStatsDelegate = FOnAdvanceRenderingThreadStats());

		/** Advances stats for commandlets, only valid if the command line has the proper token. @see HasStatsForCommandletsToken */
		static CORE_API void TickCommandletStats();

		/**
		* @return true, if the command line has the LoadTimeStatsForCommandlet or LoadTimeFileForCommandlet token which enables stats in the commandlets.
		* !!!CAUTION!!! You need to manually advance stats frame in order to maintain the data integrity and not to leak the memory.
		*/
		static CORE_API bool EnabledForCommandlet();

		/**
		* @return true, if the command line has the LoadTimeStatsForCommandlet token which enables LoadTimeStats equivalent for commandlets.
		* All collected stats will be dumped to the log file at the end of running the specified commandlet.
		*/
		static CORE_API bool HasLoadTimeStatsForCommandletToken();

		/**
		* @return true, if the command line has the LoadTimeFileForCommandlet token which enables LoadTimeFile equivalent for commandlets.
		*/
		static CORE_API bool HasLoadTimeFileForCommandletToken();

		/** Current game thread stats frame. */
		static CORE_API TAtomic<int32> GameThreadStatsFrame;

	#if STATS
		// The engine stats system maintains its own frame counter. We need to propagate this to
		// the end-of-pipe thread so that stat data it submits is attributed to the correct frame.
		// This is done inside AdvanceRenderingThreadStats.
		static CORE_API TOptional<int64> StatsFrameRT;
	#endif
	};
} // namespace UE::Stats
