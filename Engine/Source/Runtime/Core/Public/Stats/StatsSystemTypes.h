// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AutoRTFM/ReturnFromOpenMode.h"
#include "Misc/Build.h"
#include "Misc/CoreMiscDefines.h"

// Note: it would be nice to only include these when STATS is defined but many other files
// currently rely on these includes indirectly.
#include "Containers/Array.h"
#include "Containers/ChunkedArray.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/LockFreeList.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformTLS.h"
#include "HAL/PlatformTime.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/UnrealMemory.h"
#include "Math/Color.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/SourceLocation.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Stats/StatsCommon.h"
#include "Stats/StatsTrace.h"
#include "Templates/Atomic.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "Trace/Detail/Channel.inl"
#include "Trace/Trace.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

#if STATS

#include "Misc/ScopeLock.h"

class FThreadStats;
template <typename T> struct TIsPODType;

struct TStatIdData
{
	bool IsNone() const
	{
		FMinimalName LocalName = Name.Load(EMemoryOrder::Relaxed);
		return LocalName.IsNone();
	}

	TStatIdData() = default;

	TStatIdData(FMinimalName InName) : Name(InName)
	{
	}

	/** Name of the active stat; stored as a minimal name to minimize the data size */
	TAtomic<FMinimalName> Name;

	/** const ANSICHAR* pointer to a string describing the stat */
	TUniquePtr<WIDECHAR[]> StatDescriptionWide;

	/** const WIDECHAR* pointer to a string describing the stat */
	TUniquePtr<ANSICHAR[]> StatDescriptionAnsi;
};

struct TStatId
{
	// TStatId can be returned from AutoRTFM::Open() via a copy-construction in the closed.
	static constexpr AutoRTFM::EReturnFromOpenMode AutoRTFMReturnFromOpenMode = AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed;

	UE_FORCEINLINE_HINT TStatId()
		: StatIdPtr(&TStatId_NAME_None)
	{
	}
	UE_FORCEINLINE_HINT TStatId(TStatIdData const* InStatIdPtr)
		: StatIdPtr(InStatIdPtr)
	{
	}
	UE_FORCEINLINE_HINT bool IsValidStat() const
	{
		return !IsNone();
	}
	UE_FORCEINLINE_HINT bool IsNone() const
	{
		return StatIdPtr->IsNone();
	}
	UE_FORCEINLINE_HINT TStatIdData const* GetRawPointer() const
	{
		return StatIdPtr;
	}

	UE_FORCEINLINE_HINT FMinimalName GetMinimalName(EMemoryOrder MemoryOrder) const
	{
		return StatIdPtr->Name.Load(MemoryOrder);
	}

	UE_FORCEINLINE_HINT FName GetName() const
	{
		return MinimalNameToName(StatIdPtr->Name);
	}

	UE_FORCEINLINE_HINT static const TStatIdData& GetStatNone()
	{
		return TStatId_NAME_None;
	}

	/**
	 * @return a stat description as an ansi string.
	 * StatIdPtr must point to a valid FName pointer.
	 * @see FStatGroupEnableManager::GetHighPerformanceEnableForStat
	 */
	UE_FORCEINLINE_HINT const ANSICHAR* GetStatDescriptionANSI() const
	{
		return StatIdPtr->StatDescriptionAnsi.Get();
	}

	/**
	 * @return a stat description as a wide string.
	 * StatIdPtr must point to a valid FName pointer.
	 * @see FStatGroupEnableManager::GetHighPerformanceEnableForStat
	 */
	UE_FORCEINLINE_HINT const WIDECHAR* GetStatDescriptionWIDE() const
	{
		return StatIdPtr->StatDescriptionWide.Get();
	}

	UE_FORCEINLINE_HINT bool operator==(TStatId Other) const
	{
		return StatIdPtr == Other.StatIdPtr;
	}

	UE_FORCEINLINE_HINT bool operator!=(TStatId Other) const
	{
		return StatIdPtr != Other.StatIdPtr;
	}

	friend uint32 GetTypeHash(TStatId StatId)
	{
		return GetTypeHash(StatId.StatIdPtr);
	}

private:
	/** NAME_None. */
	CORE_API static TStatIdData TStatId_NAME_None;

	/**
	 *	Holds a pointer to the stat long name if enabled, or to the NAME_None if disabled.
	 *	@see FStatGroupEnableManager::EnableStat
	 *	@see FStatGroupEnableManager::DisableStat
	 *
	 *	Next pointer points to the ansi string with a stat description
	 *	Next pointer points to the wide string with a stat description
	 *	@see FStatGroupEnableManager::GetHighPerformanceEnableForStat
	 */
	TStatIdData const* StatIdPtr;
};

/**
 * For packet messages, this indicated what sort of thread timing we use.
 * Game and Other use CurrentGameFrame, Renderer is CurrentRenderFrame, EndOfPipe is CurrentEndOfPipeFrame
 */
namespace EThreadType
{
	enum Type
	{
		Invalid,
		Game,
		Renderer,
		EndOfPipe,
		Other,
	};
}


/**
 * What the type of the payload is
 */
struct EStatDataType
{
	enum Type
	{
		Invalid,
		/** Not defined. */
		ST_None,
		/** int64. */
		ST_int64,
		/** double. */
		ST_double,
		/** FName. */
		ST_FName,
		/** Memory pointer, stored as uint64. */
		ST_Ptr,

		Num,
		Mask = 0x7,
		Shift = 0,
		NumBits = 3
	};
};

/**
 * The operation being performed by this message
 */
struct EStatOperation
{
	enum Type
	{
		Invalid,
		/** Indicates metadata message. */
		SetLongName,
		/** Special message for advancing the stats frame from the game thread. */
		AdvanceFrameEventGameThread,
		/** Special message for advancing the stats frame from the render thread. */
		AdvanceFrameEventRenderThread,
		/** Special message for advancing the stats frame from the end of pipe thread. */
		AdvanceFrameEventEndOfPipe,
		/** Indicates begin of the cycle scope. */
		CycleScopeStart,
		/** Indicates end of the cycle scope. */
		CycleScopeEnd,
		/** This is not a regular stat operation, but just a special message marker to determine that we encountered a special data in the stat file. */
		SpecialMessageMarker,
		/** Set operation. */
		Set,
		/** Clear operation. */
		Clear,
		/** Add operation. */
		Add,
		/** Subtract operation. */
		Subtract,

		// these are special ones for processed data
		ChildrenStart,
		ChildrenEnd,
		Leaf,
		MaxVal,

		/** This is a memory operation. @see EMemoryOperation. */
		Memory UE_DEPRECATED(5.3, "Use Trace/MemoryInsights and/or LLM for memory profiling."),

		Num,
		Mask = 0xf,
		Shift = EStatDataType::Shift + EStatDataType::NumBits,
		NumBits = 4
	};
};

/**
 * Message flags
 */
struct EStatMetaFlags
{
	enum Type
	{
		Invalid = 0x00,
		/** this bit is always one and is used for error checking. */
		DummyAlwaysOne = 0x01,
		/** if true, then this message contains a GPU stat */
		IsGPU = 0x02,
		/** if true, then this message contains and int64 cycle or IsPackedCCAndDuration. */
		IsCycle = 0x04,
		/** if true, then this message contains a memory stat. */
		IsMemory = 0x08,
		/** if true, then this is actually two uint32s, the cycle count and the call count, see FromPackedCallCountDuration_Duration. */
		IsPackedCCAndDuration = 0x10,
		/** if true, then this stat is cleared every frame. */
		ShouldClearEveryFrame = 0x20,
		/** used only on disk on on the wire, indicates that we serialized the FName string. */
		SendingFName = 0x40,

		Num = 0x80,
		Mask = 0xff,
		Shift = EStatOperation::Shift + EStatOperation::NumBits,
		NumBits = 8
	};
};

//@todo merge these two after we have redone the user end
/**
 * Wrapper for memory region
 */
struct EMemoryRegion
{
	enum Type
	{
		Invalid = FPlatformMemory::MCR_Invalid,

		Num = FPlatformMemory::MCR_MAX,
		Mask = 0xf,
		Shift = EStatMetaFlags::Shift + EStatMetaFlags::NumBits,
		NumBits = 4
	};
	static_assert(FPlatformMemory::MCR_MAX < (1 << NumBits), "Need to expand memory region field.");
};

/**
 * A few misc final bit packing computations
 */
namespace EStatAllFields
{
	enum Type
	{
		NumBits = EMemoryRegion::Shift + EMemoryRegion::NumBits,
		StartShift = 28 - NumBits,
	};
}

static_assert(EStatAllFields::StartShift > 0, "Too many stat fields.");

UE_FORCEINLINE_HINT int64 ToPackedCallCountDuration(uint32 CallCount, uint32 Duration)
{
	return (int64(CallCount) << 32) | Duration;
}

UE_FORCEINLINE_HINT uint32 FromPackedCallCountDuration_CallCount(int64 Both)
{
	return uint32(Both >> 32);
}

UE_FORCEINLINE_HINT uint32 FromPackedCallCountDuration_Duration(int64 Both)
{
	return uint32(Both & MAX_uint32);
}

/**
 * Helper class that stores an FName and all meta information in 8 bytes. Kindof icky.
 */
class FStatNameAndInfo
{
	/**
	 * Store name and number separately in case UE_FNAME_OUTLINE_NUMBER is set, so the high bits of the Number are used for other fields.
	 */
	FNameEntryId Index;
	int32 Number;

public:
	static CORE_API const char* GpuStatCategory;

	UE_FORCEINLINE_HINT FStatNameAndInfo()
	{
	}

	/**
	 * Build from a raw FName
	 */
	inline FStatNameAndInfo(FName Other, bool bAlreadyHasMeta)
		: Index(Other.GetComparisonIndex())
		, Number(Other.GetNumber())
	{
		if (!bAlreadyHasMeta)
		{
			// ok, you can't have numbered stat FNames too large
			checkStats(!(Number >> EStatAllFields::StartShift));
			Number |= EStatMetaFlags::DummyAlwaysOne << (EStatMetaFlags::Shift + EStatAllFields::StartShift);
		}
		CheckInvariants();
	}

	/**
	 * Build with stat metadata
	 */
	inline FStatNameAndInfo(FName InStatName, char const* InGroup, char const* InCategory, TCHAR const* InDescription, EStatDataType::Type InStatType, bool bShouldClearEveryFrame, bool bCycleStat, bool bSortByName, FPlatformMemory::EMemoryCounterRegion MemoryRegion = FPlatformMemory::MCR_Invalid)
	{
		FName LongName = ToLongName(InStatName, InGroup, InCategory, InDescription, bSortByName);
		Index = LongName.GetComparisonIndex();
		Number = LongName.GetNumber();

		// ok, you can't have numbered stat FNames too large
		checkStats(!(Number >> EStatAllFields::StartShift));
		Number |= EStatMetaFlags::DummyAlwaysOne << (EStatMetaFlags::Shift + EStatAllFields::StartShift);

		SetField<EStatDataType>(InStatType);
		SetFlag(EStatMetaFlags::ShouldClearEveryFrame, bShouldClearEveryFrame);
		SetFlag(EStatMetaFlags::IsCycle, bCycleStat);
		if (MemoryRegion != FPlatformMemory::MCR_Invalid)
		{
			SetFlag(EStatMetaFlags::IsMemory, true);
			SetField<EMemoryRegion>(EMemoryRegion::Type(MemoryRegion));
		}

		if (InCategory == GpuStatCategory)
		{
			SetFlag(EStatMetaFlags::IsGPU, true);
		}

		CheckInvariants();
	}

	/**
	 * Internal use, used by the deserializer
	 */
	UE_FORCEINLINE_HINT void SetNumberDirect(int32 InNumber)
	{
		Number = InNumber;
	}

	/**
	* Internal use, used by the serializer
	 */
	inline int32 GetRawNumber() const
	{
		CheckInvariants();
		return Number;
	}

	/**
	 * Internal use by FStatsThreadState to force an update to the long name
	 */
	inline void SetRawName(FName RawName)
	{
		// ok, you can't have numbered stat FNames too large
		checkStats(!(RawName.GetNumber() >> EStatAllFields::StartShift));
		CheckInvariants();
		int32 LocalNumber = Number;
		LocalNumber &= ~((1 << EStatAllFields::StartShift) - 1);
		Index = RawName.GetComparisonIndex();
		Number = (LocalNumber | RawName.GetNumber());
	}

	/**
	 * The encoded FName with the correct, original Number
	 * The original number usually is 0
	 */
	inline FName GetRawName() const
	{
		CheckInvariants();
		return FName(Index, Index, Number & ((1 << EStatAllFields::StartShift) - 1));
	}

	/**
	 * The encoded FName with the encoded, new Number
	 * The number contains all encoded metadata
	 */
	inline FName GetEncodedName() const
	{
		CheckInvariants();
		return FName(Index, Index, Number);
	}

	/**
	 * Expensive! Extracts the shortname if this is a long name or just returns the name
	 */
	inline FName GetShortName() const
	{
		CheckInvariants();
		return GetShortNameFrom(GetRawName());
	}

	/**
	 * Expensive! Extracts the group name if this is a long name or just returns none
	 */
	inline FName GetGroupName() const
	{
		CheckInvariants();
		return GetGroupNameFrom(GetRawName());
	}

	/**
	 * Expensive! Extracts the group category if this is a long name or just returns none
	 */
	inline FName GetGroupCategory() const
	{
		CheckInvariants();
		return GetGroupCategoryFrom(GetRawName());
	}

	/**
	 * Expensive! Extracts the description if this is a long name or just returns the empty string
	 */
	inline FString GetDescription() const
	{
		CheckInvariants();
		return GetDescriptionFrom(GetRawName());
	}
	/**
	 * Expensive! Extracts the sort by name flag
	 */
	inline bool GetSortByName() const
	{
		CheckInvariants();
		return GetSortByNameFrom(GetRawName());
	}


	/**
	 * Makes sure this object is in good shape
	 */
	inline void CheckInvariants() const
	{
		checkStats((Number & (EStatMetaFlags::DummyAlwaysOne << (EStatAllFields::StartShift + EStatMetaFlags::Shift)))
			&& Index);
	}

	/**
	 * returns an encoded field
	 * @return the field
	 */
	template<typename TField>
	typename TField::Type GetField() const
	{
		CheckInvariants();
		int32 LocalNumber = Number;
		LocalNumber = (LocalNumber >> (EStatAllFields::StartShift + TField::Shift)) & TField::Mask;
		checkStats(LocalNumber != TField::Invalid && LocalNumber < TField::Num);
		return typename TField::Type(LocalNumber);
	}

	/**
	 * sets an encoded field
	 * @param Value, value to set
	 */
	template<typename TField>
	void SetField(typename TField::Type Value)
	{
		int32 LocalNumber = Number;
		CheckInvariants();
		checkStats(Value < TField::Num && Value != TField::Invalid);
		LocalNumber &= ~(TField::Mask << (EStatAllFields::StartShift + TField::Shift));
		LocalNumber |= Value << (EStatAllFields::StartShift + TField::Shift);
		Number = LocalNumber;
		CheckInvariants();
	}

	/**
	 * returns an encoded flag
	 * @param Bit, flag to read
	 */
	bool GetFlag(EStatMetaFlags::Type Bit) const
	{
		int32 LocalNumber = Number;
		CheckInvariants();
		checkStats(Bit < EStatMetaFlags::Num && Bit != EStatMetaFlags::Invalid);
		return !!((LocalNumber >> (EStatAllFields::StartShift + EStatMetaFlags::Shift)) & Bit);
	}

	/**
	 * sets an encoded flag
	 * @param Bit, flag to set
	 * @param Value, value to set
	 */
	void SetFlag(EStatMetaFlags::Type Bit, bool Value)
	{
		int32 LocalNumber = Number;
		CheckInvariants();
		checkStats(Bit < EStatMetaFlags::Num && Bit != EStatMetaFlags::Invalid);
		if (Value)
		{
			LocalNumber |= (Bit << (EStatAllFields::StartShift + EStatMetaFlags::Shift));
		}
		else
		{
			LocalNumber &= ~(Bit << (EStatAllFields::StartShift + EStatMetaFlags::Shift));
		}
		Number = LocalNumber;
		CheckInvariants();
	}


	/**
	 * Builds a long name from the three parts
	 * @param InStatName, Short name
	 * @param InGroup, Group name
	 * @param InCategory, Category name
	 * @param InDescription, Description
	 * @param InSortByName, Whether this stats need to be sorted by name
	 * @return the packed FName
	 */
	CORE_API static FName ToLongName(FName InStatName, char const* InGroup, char const* InCategory, TCHAR const* InDescription, bool InSortByName);
	CORE_API static FName GetShortNameFrom(FName InLongName);
	CORE_API static FName GetGroupNameFrom(FName InLongName);
	CORE_API static FName GetGroupCategoryFrom(FName InLongName);
	CORE_API static FString GetDescriptionFrom(FName InLongName);
	CORE_API static bool GetSortByNameFrom(FName InLongName);

};


/** Union for easier debugging. */
union UStatData
{
private:
	/** For ST_double. */
	double	Float;
	/** For ST_int64 and IsCycle or IsMemory. */
	int64	Cycles;
	/** For ST_Ptr. */
	uint64	Ptr;
	/** ST_int64 and IsPackedCCAndDuration. */
	uint32	CCAndDuration[2];
	/** For FName. */
	CORE_API const FString GetName() const
	{
		return FName::SafeString(FNameEntryId::FromUnstableInt(static_cast<uint32>(Cycles)));
	}
};

/**
* 16 byte stat message. Everything is a message
*/
struct FStatMessage
{
	/**
	* Generic payload
	*/
	enum
	{
		DATA_SIZE = 8,
		DATA_ALIGN = 8,
	};
	union
	{
#if	UE_BUILD_DEBUG
		UStatData								DebugStatData;
#endif // UE_BUILD_DEBUG
		TAlignedBytes<DATA_SIZE, DATA_ALIGN>	StatData;
	};

	/**
	* Name and the meta info.
	*/
	FStatNameAndInfo						NameAndInfo;

	FStatMessage()
	{
	}

	/**
	* Build a meta data message
	*/
	FStatMessage(FName InStatName, EStatDataType::Type InStatType, char const* InGroup, char const* InCategory, TCHAR const* InDescription, bool bShouldClearEveryFrame, bool bCycleStat, bool bSortByName, FPlatformMemory::EMemoryCounterRegion MemoryRegion = FPlatformMemory::MCR_Invalid)
		: NameAndInfo(InStatName, InGroup, InCategory, InDescription, InStatType, bShouldClearEveryFrame, bCycleStat, bSortByName, MemoryRegion)
	{
		NameAndInfo.SetField<EStatOperation>(EStatOperation::SetLongName);
	}

	explicit UE_FORCEINLINE_HINT FStatMessage(FStatNameAndInfo InStatName)
		: NameAndInfo(InStatName)
	{
	}

	/**
	* Clock operation
	*/
	inline FStatMessage(FName InStatName, EStatOperation::Type InStatOperation)
		: NameAndInfo(InStatName, true)
	{
		NameAndInfo.SetField<EStatOperation>(InStatOperation);
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64);
		checkStats(NameAndInfo.GetFlag(EStatMetaFlags::IsCycle) == true);

		// these branches are FORCEINLINE_STATS of constants in almost all cases, so they disappear
		if (InStatOperation == EStatOperation::CycleScopeStart || InStatOperation == EStatOperation::CycleScopeEnd)
		{
			GetValue_int64() = int64(FPlatformTime::Cycles());
		}
		else
		{
			checkStats(0);
		}
	}

	/**
	* int64 operation
	*/
	inline FStatMessage(FName InStatName, EStatOperation::Type InStatOperation, int64 Value, bool bIsCycle)
		: NameAndInfo(InStatName, true)
	{
		NameAndInfo.SetField<EStatOperation>(InStatOperation);
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64);
		checkStats(NameAndInfo.GetFlag(EStatMetaFlags::IsCycle) == bIsCycle);
		GetValue_int64() = Value;
	}

	/**
	* double operation
	*/
	inline FStatMessage(FName InStatName, EStatOperation::Type InStatOperation, double Value, bool)
		: NameAndInfo(InStatName, true)
	{
		NameAndInfo.SetField<EStatOperation>(InStatOperation);
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_double);
		checkStats(NameAndInfo.GetFlag(EStatMetaFlags::IsCycle) == false);
		GetValue_double() = Value;
	}

	/**
	* name operation
	*/
	inline FStatMessage(FName InStatName, EStatOperation::Type InStatOperation, FName Value, bool)
		: NameAndInfo(InStatName, true)
	{
		NameAndInfo.SetField<EStatOperation>(InStatOperation);
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_FName);
		checkStats(NameAndInfo.GetFlag(EStatMetaFlags::IsCycle) == false);
		GetValue_FMinimalName() = NameToMinimalName(Value);
	}

	/**
	 * Ptr operation
	 */
	inline FStatMessage(FName InStatName, EStatOperation::Type InStatOperation, uint64 Value, bool)
		: NameAndInfo(InStatName, true)
	{
		NameAndInfo.SetField<EStatOperation>(InStatOperation);
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_Ptr);
		checkStats(NameAndInfo.GetFlag(EStatMetaFlags::IsCycle) == false);
		GetValue_Ptr() = Value;
	}

	/**
	* Clear any data type
	*/
	inline void Clear()
	{
		static_assert(sizeof(uint64) == DATA_SIZE, "Bad clear.");
		*(int64*)&StatData = 0;
	}

	/**
	* Payload retrieval and setting methods
	*/
	inline int64& GetValue_int64()
	{
		static_assert(sizeof(int64) <= DATA_SIZE && alignof(int64) <= DATA_ALIGN, "Bad data for stat message.");
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64);
		return *(int64*)&StatData;
	}
	inline int64 GetValue_int64() const
	{
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64);
		return *(int64 const*)&StatData;
	}

	inline uint64& GetValue_Ptr()
	{
		static_assert(sizeof(uint64) <= DATA_SIZE && alignof(uint64) <= DATA_ALIGN, "Bad data for stat message.");
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_Ptr);
		return *(uint64*)&StatData;
	}
	inline uint64 GetValue_Ptr() const
	{
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_Ptr);
		return *(uint64 const*)&StatData;
	}

	inline int64 GetValue_Duration() const
	{
		checkStats(NameAndInfo.GetFlag(EStatMetaFlags::IsCycle) && NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64);
		if (NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration))
		{
			return FromPackedCallCountDuration_Duration(*(int64 const*)&StatData);
		}
		return *(int64 const*)&StatData;
	}

	inline uint32 GetValue_CallCount() const
	{
		checkStats(NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration) && NameAndInfo.GetFlag(EStatMetaFlags::IsCycle) && NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64);
		return FromPackedCallCountDuration_CallCount(*(int64 const*)&StatData);
	}

	inline double& GetValue_double()
	{
		static_assert(sizeof(double) <= DATA_SIZE && alignof(double) <= DATA_ALIGN, "Bad data for stat message.");
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_double);
		return *(double*)&StatData;
	}

	inline double GetValue_double() const
	{
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_double);
		return *(double const*)&StatData;
	}

	inline FMinimalName& GetValue_FMinimalName()
	{
		static_assert(sizeof(FMinimalName) <= DATA_SIZE && alignof(FMinimalName) <= DATA_ALIGN, "Bad data for stat message.");
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_FName);
		return *(FMinimalName*)&StatData;
	}

	inline FMinimalName GetValue_FMinimalName() const
	{
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_FName);
		return *(FMinimalName const*)&StatData;
	}

	inline FName GetValue_FName() const
	{
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_FName);
		return MinimalNameToName(*(FMinimalName const*)&StatData);
	}
};
template<> struct TIsPODType<FStatMessage> { enum { Value = true }; };

CORE_API void GetPermanentStats(TArray<FStatMessage>& OutStats);

/**
 *	Based on FStatMessage, but supports more than 8 bytes of stat data storage.
 */
template< typename TEnum >
struct TStatMessage
{
	typedef TEnum TStructEnum;
	static constexpr int32 EnumCount = TEnum::Num;

	/**
	* Generic payload
	*/
	enum
	{
		DATA_SIZE = 8 * EnumCount,
		DATA_ALIGN = 8,
	};

	union
	{
#if	UE_BUILD_DEBUG
		UStatData								DebugStatData[EnumCount];
#endif // UE_BUILD_DEBUG
		TAlignedBytes<DATA_SIZE, DATA_ALIGN>	StatData;
	};

	/**
	* Name and the meta info.
	*/
	FStatNameAndInfo							NameAndInfo;

	TStatMessage()
	{}

	/**
	* Copy constructor from FStatMessage
	*/
	explicit inline TStatMessage(const FStatMessage& Other)
		: NameAndInfo(Other.NameAndInfo)
	{
		// Reset data type and clear all fields.
		NameAndInfo.SetField<EStatDataType>(EStatDataType::ST_None);
		Clear();
	}

	/** Assignment operator for FStatMessage. */
	TStatMessage& operator=(const FStatMessage& Other)
	{
		NameAndInfo = Other.NameAndInfo;
		// Reset data type and clear all fields.
		NameAndInfo.SetField<EStatDataType>(EStatDataType::ST_None);
		Clear();
		return *this;
	}

	/** Fixes stat data type for all fields. */
	void FixStatData(const EStatDataType::Type NewType)
	{
		const EStatDataType::Type OldType = NameAndInfo.GetField<EStatDataType>();
		if (OldType != NewType)
		{
			// Convert from the old type to the new type.
			if (OldType == EStatDataType::ST_int64 && NewType == EStatDataType::ST_double)
			{
				// Get old values.
				int64 OldValues[TEnum::Num];
				for (int32 FieldIndex = 0; FieldIndex < EnumCount; ++FieldIndex)
				{
					OldValues[FieldIndex] = GetValue_int64((typename TEnum::Type)FieldIndex);
				}

				// Set new stat data type
				NameAndInfo.SetField<EStatDataType>(NewType);
				for (int32 FieldIndex = 0; FieldIndex < EnumCount; ++FieldIndex)
				{
					GetValue_double((typename TEnum::Type)FieldIndex) = (double)OldValues[FieldIndex];
				}
			}
			else if (OldType == EStatDataType::ST_double && NewType == EStatDataType::ST_int64)
			{
				// Get old values.
				double OldValues[TEnum::Num];
				for (int32 FieldIndex = 0; FieldIndex < EnumCount; ++FieldIndex)
				{
					OldValues[FieldIndex] = GetValue_double((typename TEnum::Type)FieldIndex);
				}

				// Set new stat data type
				NameAndInfo.SetField<EStatDataType>(NewType);
				for (int32 FieldIndex = 0; FieldIndex < EnumCount; ++FieldIndex)
				{
					GetValue_int64((typename TEnum::Type)FieldIndex) = (int64)OldValues[FieldIndex];
				}
			}
		}
	}

	/**
	* Clear any data type
	*/
	inline void Clear()
	{
		static_assert(sizeof(uint64) == DATA_SIZE / EnumCount, "Bad clear.");

		for (int32 FieldIndex = 0; FieldIndex < EnumCount; ++FieldIndex)
		{
			*((int64*)&StatData + FieldIndex) = 0;
		}
	}

	/**
	* Payload retrieval and setting methods
	*/
	inline int64& GetValue_int64(typename TEnum::Type Index)
	{
		static_assert(sizeof(int64) <= DATA_SIZE && alignof(int64) <= DATA_ALIGN, "Bad data for stat message.");
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64);
		checkStats(Index < EnumCount);
		int64& Value = *((int64*)&StatData + (uint32)Index);
		return Value;
	}
	inline int64 GetValue_int64(typename TEnum::Type Index) const
	{
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64);
		checkStats(Index < EnumCount);
		const int64 Value = *((int64*)&StatData + (uint32)Index);
		return Value;
	}

	inline int64 GetValue_Duration(typename TEnum::Type Index) const
	{
		checkStats(NameAndInfo.GetFlag(EStatMetaFlags::IsCycle) && NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64);
		checkStats(Index < EnumCount);
		if (NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration))
		{
			const uint32 Value = FromPackedCallCountDuration_Duration(*((int64 const*)&StatData + (uint32)Index));
			return Value;
		}
		const int64 Value = *((int64 const*)&StatData + (uint32)Index);
		return Value;
	}

	inline uint32 GetValue_CallCount(typename TEnum::Type Index) const
	{
		checkStats(NameAndInfo.GetFlag(EStatMetaFlags::IsPackedCCAndDuration) && NameAndInfo.GetFlag(EStatMetaFlags::IsCycle) && NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_int64);
		checkStats(Index < EnumCount);
		const uint32 Value = FromPackedCallCountDuration_CallCount(*((int64 const*)&StatData + (uint32)Index));
		return Value;
	}

	inline double& GetValue_double(typename TEnum::Type Index)
	{
		static_assert(sizeof(double) <= DATA_SIZE && alignof(double) <= DATA_ALIGN, "Bad data for stat message.");
		checkStats(Index < EnumCount);
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_double);
		double& Value = *((double*)&StatData + (uint32)Index);
		return Value;
	}

	inline double GetValue_double(typename TEnum::Type Index) const
	{
		checkStats(NameAndInfo.GetField<EStatDataType>() == EStatDataType::ST_double);
		checkStats(Index < EnumCount);
		const double Value = *((double const*)&StatData + (uint32)Index);
		return Value;
	}

	UE_FORCEINLINE_HINT FName GetShortName() const
	{
		return NameAndInfo.GetShortName();
	}

	UE_FORCEINLINE_HINT FString GetDescription() const
	{
		return NameAndInfo.GetDescription();
	}
};

/** Enumerates fields of the FComplexStatMessage. */
struct EComplexStatField
{
	enum Type
	{
		/** Summed inclusive time. */
		IncSum,
		/** Average inclusive time. */
		IncAve,
		/** Maximum inclusive time. */
		IncMax,
		/** Minimum inclusive time. */
		IncMin,
		/** Summed exclusive time. */
		ExcSum,
		/** Average exclusive time. */
		ExcAve,
		/** Maximum exclusive time. */
		ExcMax,
		/** Minimum exclusive time. */
		ExcMin,

		/** Number of enumerates. */
		Num,
	};
};

/**
 *	This type of stat message holds data defined by associated enumeration @see EComplexStatField
 *	By default any of these messages contain a valid data, so check before accessing the data.
 */
typedef TStatMessage<EComplexStatField> FComplexStatMessage;

template<> struct TIsPODType<FComplexStatMessage> { enum { Value = true }; };


struct EStatMessagesArrayConstants
{
	enum
	{
#if WITH_EDITOR
		MESSAGES_CHUNK_SIZE = 4 * 1024, // Smaller chunks prevent malloc deadlocks when running in the editor with TBB allocator
#else
		MESSAGES_CHUNK_SIZE = 64 * 1024,
#endif
	};
};
typedef TChunkedArray<FStatMessage, (uint32)EStatMessagesArrayConstants::MESSAGES_CHUNK_SIZE> FStatMessagesArray;

/**
* A stats packet. Sent between threads. Includes and array of messages and some information about the thread.
*/
struct FStatPacket
{
	/** Assigned later, this is the frame number this packet is for. @see FStatsThreadState::ScanForAdvance or FThreadStats::DetectAndUpdateCurrentGameFrame */
	int64 Frame = 1;
	/** ThreadId this packet came from **/
	uint32 ThreadId = 0;
	/** type of thread this packet came from **/
	EThreadType::Type ThreadType;
	/** true if this packet has broken callstacks **/
	bool bBrokenCallstacks = false;
	/** messages in this packet **/
	FStatMessagesArray StatMessages;
	/** Size we presize the message buffer to, currently the max of what we have seen for the last PRESIZE_MAX_NUM_ENTRIES. **/
	TArray<int32> StatMessagesPresize;

	/** constructor **/
	FStatPacket(EThreadType::Type InThreadType = EThreadType::Invalid)
		: ThreadType(InThreadType)
	{
	}

	/** Copy constructor. !!!CAUTION!!! does not copy the data **/
	FStatPacket(FStatPacket const& Other)
		: Frame(Other.Frame)
		, ThreadId(Other.ThreadId)
		, ThreadType(Other.ThreadType)
		, bBrokenCallstacks(false)
		, StatMessagesPresize(Other.StatMessagesPresize)
	{
	}

	/** Initializes thread related properties for the stats packet. */
	void SetThreadProperties()
	{
		ThreadId = FPlatformTLS::GetCurrentThreadId();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (ThreadId == GGameThreadId)
			{
				ThreadType = EThreadType::Game;
			}
			else if (ThreadId == GRenderThreadId)
			{
				ThreadType = EThreadType::Renderer;
			}
			else
			{
				ThreadType = EThreadType::Other;
			}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void AssignFrame(int64 InFrame)
	{
		Frame = InFrame;
	}
};

/** Helper struct used to monitor the scope of the message. */
struct FStatMessageLock
{
	FStatMessageLock(int32& InMessageScope)
		: MessageScope(InMessageScope)
	{
		MessageScope++;
	}

	~FStatMessageLock()
	{
		MessageScope--;
	}

protected:
	int32& MessageScope;
};

/** Preallocates FThreadStats to avoid dynamic memory allocation. */
struct FThreadStatsPool
{
private:
	/** Lock free pool of FThreadStats instances. */
	TLockFreePointerListUnordered<void, 0> Pool;

public:
	/** Default constructor. */
	CORE_API FThreadStatsPool();

	/** Singleton accessor. */
	CORE_API static FThreadStatsPool& Get();

	/** Gets an instance from the pool and call the default constructor on it. */
	CORE_API FThreadStats* GetFromPool();

	/** Return an instance to the pool. */
	CORE_API void ReturnToPool(FThreadStats* Instance);
};

/**
* This is thread-private information about the stats we are acquiring. Pointers to these objects are held in TLS.
*/
class FThreadStats : FNoncopyable
{
	friend class FStatsMallocProfilerProxy;
	friend class FStatsThreadState;
	friend class FStatsThread;
	friend struct FThreadStatsPool;

	/** Used to control when we are collecting stats. User of the stats system increment and decrement this counter as they need data. **/
	CORE_API static FThreadSafeCounter PrimaryEnableCounter;
	/** Every time bPrimaryEnable changes, we update this. This is used to determine frames that have complete data. **/
	CORE_API static FThreadSafeCounter PrimaryEnableUpdateNumber;
	/** while bPrimaryEnable (or other things affecting stat collection) is chaning, we lock this. This is used to determine frames that have complete data. **/
	CORE_API static FThreadSafeCounter PrimaryDisableChangeTagLock;
	/** TLS slot that holds a FThreadStats. **/
	CORE_API static uint32 TlsSlot;
	/** Computed by CheckEnable, the current "primary control" for stats collection, based on PrimaryEnableCounter and a few other things. **/
	CORE_API static bool bPrimaryEnable;
	/** Set to permanently disable the stats system. **/
	CORE_API static bool bPrimaryDisableForever;
	/** True if we running in the raw stats mode, all stats processing is disabled, captured stats messages are written in timely manner, memory overhead is minimal. */
	CORE_API static bool bIsRawStatsActive;

	/** The data we are eventually going to send to the stats thread. **/
	FStatPacket Packet;

	/** Current game frame for this thread stats. */
	int32 CurrentGameFrame;

	/** Tracks current stack depth for cycle counters. **/
	int32 ScopeCount = 0;

	/** Tracks current stack depth for cycle counters. **/
	int32 bWaitForExplicitFlush = 0;

	/**
	 * Tracks the progress of adding a new memory stat message and prevents from using the memory profiler in the scope.
	 * Mostly related to ignoring all memory allocations in AddStatMessage().
	 * Memory usage of the stats messages is handled by the STAT_StatMessagesMemory.
	 */
	int32 MemoryMessageScope = 0;

	/** We have a scoped cycle counter in the FlushRawStats which causes an infinite recursion, ReentranceGuard will solve that issue. */
	bool bReentranceGuard = false;

	/** Tracks current stack depth for cycle counters. **/
	bool bSawExplicitFlush = false;

	void SendMessage_Async(FStatPacket* ToSend);

protected:
	/** Gathers information about the current thread and sets up the TLS value. **/
	CORE_API FThreadStats(EThreadType::Type InThreadType = EThreadType::Invalid);

public:
	/** Checks the TLS for a thread packet and if it isn't found, it makes a new one. **/
	static inline FThreadStats* GetThreadStats()
	{
		FThreadStats* Stats = (FThreadStats*)FPlatformTLS::GetTlsValue(TlsSlot);
		if (!Stats)
		{
			Stats = FThreadStatsPool::Get().GetFromPool();
		}
		return Stats;
	}

	/** This should be called when conditions have changed such that stat collection may now be enabled or not **/
	static CORE_API void CheckEnable();

	/**
	 *	Checks if the game frame has changed and updates the current game frame.
	 *	Used by the flushing mechanism to optimize the memory usage, only valid for packets from other threads.
	 *	@return true, if the frame has changed
	 */
	CORE_API bool DetectAndUpdateCurrentGameFrame();

	/** Maintains the explicit flush. */
	inline void UpdateExplicitFlush()
	{
		if (Packet.ThreadType != EThreadType::Other && bSawExplicitFlush)
		{
			bSawExplicitFlush = false;
			bWaitForExplicitFlush = 1;
			ScopeCount++; // prevent sends until the next explicit flush
		}
	}

	/** Send any outstanding packets to the stats thread. **/
	CORE_API void Flush(bool bHasBrokenCallstacks = false, bool bForceFlush = false);

	/** Flushes the regular stats, the realtime stats. */
	CORE_API void FlushRegularStats(bool bHasBrokenCallstacks, bool bForceFlush);

	/** Flushes the raw stats, low memory and performance overhead, but not realtime. */
	CORE_API void FlushRawStats(bool bHasBrokenCallstacks = false, bool bForceFlush = false);

	/** Checks the command line whether we want to enable collecting startup stats. */
	static CORE_API void CheckForCollectingStartupStats();

	UE_AUTORTFM_ALWAYS_OPEN
	inline void AddStatMessage(const FStatMessage& StatMessage)
	{
		LLM_SCOPE(ELLMTag::Stats);
		FStatMessageLock MessageLock(MemoryMessageScope);
		Packet.StatMessages.AddElement(StatMessage);
	}

protected:
	/** Any non-clock operation with an ordinary payload. **/
	template <typename TValue>
	inline void AddMessageInner(FName InStatName, EStatOperation::Type InStatOperation, TValue Value, bool bIsCycle = false)
	{
		AddStatMessage(FStatMessage(InStatName, InStatOperation, Value, bIsCycle));

		if constexpr (std::is_same_v<std::decay_t<TValue>, double> || std::is_same_v<std::decay_t<TValue>, int64>)
		{
			switch (InStatOperation)
			{
			case EStatOperation::Set:      TRACE_STAT_SET(InStatName,  Value); break;
			case EStatOperation::Add:      TRACE_STAT_ADD(InStatName,  Value); break;
			case EStatOperation::Subtract: TRACE_STAT_ADD(InStatName, -Value); break;
			}
		}

		if (!ScopeCount)
		{
			Flush();
		}
		else if (bIsRawStatsActive)
		{
			FlushRawStats();
		}
	}

	/** Clock operation. **/
	UE_AUTORTFM_ALWAYS_OPEN
	inline void AddMessageInner(FName InStatName, EStatOperation::Type InStatOperation)
	{
		checkStats((InStatOperation == EStatOperation::CycleScopeStart || InStatOperation == EStatOperation::CycleScopeEnd));

		// these branches are handled by the optimizer
		if (InStatOperation == EStatOperation::CycleScopeStart)
		{
			ScopeCount++;
			AddStatMessage(FStatMessage(InStatName, InStatOperation));

			if (bIsRawStatsActive)
			{
				FlushRawStats();
			}
		}
		else if (InStatOperation == EStatOperation::CycleScopeEnd)
		{
			if (ScopeCount > bWaitForExplicitFlush)
			{
				AddStatMessage(FStatMessage(InStatName, InStatOperation));
				ScopeCount--;
				if (!ScopeCount)
				{
					Flush();
				}
				else if (bIsRawStatsActive)
				{
					FlushRawStats();
				}
			}
			// else we dumped this frame without closing scope, so we just drop the closes on the floor
		}
	}

public:
	/** This should be called when a thread exits, this deletes FThreadStats from the heap and TLS. **/
	static void Shutdown()
	{
		FThreadStats* Stats = IsThreadingReady() ? (FThreadStats*)FPlatformTLS::GetTlsValue(TlsSlot) : nullptr;
		if (Stats)
		{
			// Send all remaining messages.
			Stats->Flush(false, true);
			FPlatformTLS::SetTlsValue(TlsSlot, nullptr);
			FThreadStatsPool::Get().ReturnToPool(Stats);
		}
	}

	/** Clock operation. **/
	static inline void AddMessage(FName InStatName, EStatOperation::Type InStatOperation)
	{
		if (!InStatName.IsNone() && WillEverCollectData() && IsThreadingReady())
		{
			GetThreadStats()->AddMessageInner(InStatName, InStatOperation);
		}
	}

	/** Any non-clock operation with an ordinary payload. **/
	template<typename TValue>
	static inline void AddMessage(FName InStatName, EStatOperation::Type InStatOperation, TValue Value, bool bIsCycle = false)
	{
		if (!InStatName.IsNone() && WillEverCollectData() && IsThreadingReady())
		{
			GetThreadStats()->AddMessageInner(InStatName, InStatOperation, Value, bIsCycle);
		}
	}

	/**
	 * Used to force a flush at the next available opportunity. This is not useful for threads other than the main and render thread.
	 * if DiscardCallstack is true, we also dump call stacks, making the next available opportunity at the next stat or stat close.
	**/
	static CORE_API void ExplicitFlush(bool DiscardCallstack = false);

	/** Return true if we are currently collecting data **/
	static UE_FORCEINLINE_HINT bool IsCollectingData()
	{
		return bPrimaryEnable;
	}
	static UE_FORCEINLINE_HINT bool IsCollectingData(TStatId StatId)
	{
		// we don't test StatId for nullptr here because we assume it is non-null. If it is nullptr, that indicates a problem with higher level code.
		return !StatId.IsNone() && IsCollectingData();
	}

	/** Return true if we are currently collecting data **/
	static UE_FORCEINLINE_HINT bool WillEverCollectData()
	{
		return !bPrimaryDisableForever;
	}

	/** Return true if the threading is ready **/
	static UE_FORCEINLINE_HINT bool IsThreadingReady()
	{
		return FPlatformTLS::IsValidTlsSlot(TlsSlot);
	}

	/** Indicate that you would like the system to begin collecting data, if it isn't already collecting data. Think reference count. **/
	static inline void PrimaryEnableAdd(int32 Value = 1)
	{
		PrimaryEnableCounter.Add(Value);
		CheckEnable();
	}

	/** Indicate that you no longer need stat data, if nobody else needs stat data, then no stat data will be collected. Think reference count. **/
	static inline void PrimaryEnableSubtract(int32 Value = 1)
	{
		PrimaryEnableCounter.Subtract(Value);
		CheckEnable();
	}

	/** Indicate that you no longer need stat data, forever. **/
	static inline void PrimaryDisableForever()
	{
		bPrimaryDisableForever = true;
		CheckEnable();
	}

	/** This is called before we start to change something that will invalidate. **/
	static inline void PrimaryDisableChangeTagLockAdd(int32 Value = 1)
	{
		PrimaryDisableChangeTagLock.Add(Value);
		FPlatformMisc::MemoryBarrier();
		PrimaryEnableUpdateNumber.Increment();
	}

	/** Indicate that you no longer need stat data, if nobody else needs stat data, then no stat data will be collected. Think reference count. **/
	static inline void PrimaryDisableChangeTagLockSubtract(int32 Value = 1)
	{
		FPlatformMisc::MemoryBarrier();
		PrimaryEnableUpdateNumber.Increment();
		FPlatformMisc::MemoryBarrier();
		PrimaryDisableChangeTagLock.Subtract(Value);
	}

	/** Everytime primary enable changes, this number increases. This is used to determine full frames. **/
	static inline int32 PrimaryDisableChangeTag()
	{
		if (PrimaryDisableChangeTagLock.GetValue())
		{
			// while locked we are continually invalid, so we will just keep giving unique numbers
			return PrimaryEnableUpdateNumber.Increment();
		}
		return PrimaryEnableUpdateNumber.GetValue();
	}

	/** Call this if something disrupts data gathering. For example when the render thread is killed, data is abandoned.**/
	static inline void FrameDataIsIncomplete()
	{
		FPlatformMisc::MemoryBarrier();
		PrimaryEnableUpdateNumber.Increment();
		FPlatformMisc::MemoryBarrier();
	}

	/** Enables the raw stats mode. */
	static inline void EnableRawStats() TSAN_SAFE
	{
		bIsRawStatsActive = true;
		FPlatformMisc::MemoryBarrier();
	}

	/** Disables the raw stats mode. */
	static inline void DisableRawStats() TSAN_SAFE
	{
		bIsRawStatsActive = false;
		FPlatformMisc::MemoryBarrier();
	}

	/** Called by launch engine loop to start the stats thread **/
	static CORE_API void StartThread();
	/** Called by launch engine loop to stop the stats thread **/
	static CORE_API void StopThread();
	/** Called by the engine loop to make sure the stats thread isn't getting too far behind. **/
	static CORE_API void WaitForStats();
};

/*
 * Wrapper used by the end-of-pipe tasks to report stats on the appropriate timeline.
 * Acts as a singleton instance of FThreadStats, since the end-of-pipe is a logical part of the frame, not a specific thread.
 */
class FEndOfPipeStats : private FThreadStats
{
	static CORE_API FEndOfPipeStats EndOfPipeStats;

	// Serializes producers writing to the singleton's inherited Packet.
	// Multiple workers (RHI submission thread + task-graph EOP tasks) call
	// AddMessage on this instance; without this lock the chunked array writes
	// race and produce torn FStatMessages that fail the DummyAlwaysOne check
	// in FStatsThreadState::ProcessNonFrameStats.
	FTransactionallySafeCriticalSection PacketCriticalSection;

	FEndOfPipeStats()
		: FThreadStats(EThreadType::EndOfPipe)
	{}

public:
	static FEndOfPipeStats* Get()
	{
		return &EndOfPipeStats;
	}

	template<typename TValue>
	void AddMessage(FName InStatName, EStatOperation::Type InStatOperation, TValue Value, bool bIsCycle = false)
	{
		if (!InStatName.IsNone() && WillEverCollectData() && IsThreadingReady())
		{
			UE::TScopeLock Lock(PacketCriticalSection);
			FThreadStats::AddMessageInner(InStatName, InStatOperation, Value, bIsCycle);
		}
	}

	void Flush()
	{
		UE::TScopeLock Lock(PacketCriticalSection);
		FThreadStats::Flush(false, true);
	}
};

/**
 * This is a utility class for counting the number of cycles during the
 * lifetime of the object. It creates messages for the stats thread.
 */
class FCycleCounter
{
	/** Event types emitted */
	enum
	{
		NamedEventFlag = 1 << 0,
		TraceEventFlag = 1 << 1,
		ThreadStatsEventFlag = 1 << 2,
	};

	/** Name of the stat, usually a short name **/
	FName StatId;
	uint8 EmittedEvent = 0;

public:

	/**
	 * Pushes the specified stat onto the hierarchy for this thread. Starts
	 * the timing of the cycles used
	 */
	UE_AUTORTFM_ALWAYS_OPEN
	inline void Start(TStatId InStatId, EStatFlags InStatFlags, bool bAlways = false, UE::FSourceLocation SourceLocation = UE::FSourceLocation::Current())
	{
		FMinimalName StatMinimalName = InStatId.GetMinimalName(EMemoryOrder::Relaxed);
		if (StatMinimalName.IsNone())
		{
			return;
		}

		// Emit named event for active cycle stat.
		if (GCycleStatsShouldEmitNamedEvents
			&& (GShouldEmitVerboseNamedEvents || !EnumHasAnyFlags(InStatFlags, EStatFlags::Verbose)))
		{
#if PLATFORM_USES_ANSI_STRING_FOR_EXTERNAL_PROFILING
			FPlatformMisc::BeginNamedEvent(FColor(0), InStatId.GetStatDescriptionANSI());
#else
			FPlatformMisc::BeginNamedEvent(FColor(0), InStatId.GetStatDescriptionWIDE());
#endif
			EmittedEvent |= NamedEventFlag;

#if CPUPROFILERTRACE_ENABLED
			if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
			{
				FName StatName = MinimalNameToName(StatMinimalName);
				FCpuProfilerTrace::OutputBeginDynamicEventWithId(StatName, InStatId.GetStatDescriptionWIDE(), SourceLocation.GetFileName(), SourceLocation.GetLine());
				EmittedEvent |= TraceEventFlag;
			}
#endif
		}

		if ((bAlways && FThreadStats::WillEverCollectData()) || FThreadStats::IsCollectingData())
		{
			FName StatName = MinimalNameToName(StatMinimalName);
			StatId = StatName;
			FThreadStats::AddMessage(StatName, EStatOperation::CycleScopeStart);
			EmittedEvent |= ThreadStatsEventFlag;
		}
	}

	UE_AUTORTFM_ALWAYS_OPEN
	UE_FORCEINLINE_HINT void Start(TStatId InStatId, bool bAlways = false, UE::FSourceLocation SourceLocation = UE::FSourceLocation::Current())
	{
		Start(InStatId, EStatFlags::None, bAlways, SourceLocation);
	}

	UE_AUTORTFM_ALWAYS_OPEN
	inline void StartTrace(const FName Name)
	{
#if CPUPROFILERTRACE_ENABLED
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
		{
			FCpuProfilerTrace::OutputBeginDynamicEvent(Name);
			EmittedEvent |= TraceEventFlag;
		}
#endif
	}

	UE_AUTORTFM_ALWAYS_OPEN
	inline void StartTrace(const FName Name, const TCHAR* Desc)
	{
#if CPUPROFILERTRACE_ENABLED
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
		{
			FCpuProfilerTrace::OutputBeginDynamicEventWithId(Name, Desc);
			EmittedEvent |= TraceEventFlag;
		}
#endif
	}

	/**
	 * Stops the capturing and stores the result
	 */
	UE_AUTORTFM_ALWAYS_OPEN
	inline void Stop()
	{
		if (EmittedEvent & NamedEventFlag)
		{
			FPlatformMisc::EndNamedEvent();
		}

#if CPUPROFILERTRACE_ENABLED
		if (EmittedEvent & TraceEventFlag)
		{
			FCpuProfilerTrace::OutputEndEvent();
		}
#endif

		if (EmittedEvent & ThreadStatsEventFlag)
		{
			FThreadStats::AddMessage(StatId, EStatOperation::CycleScopeEnd);
		}

		EmittedEvent = 0;
	}

	/**
	 * Stops the capturing and stores the result and resets the stat id.
	 */
	UE_AUTORTFM_ALWAYS_OPEN
	inline void StopAndResetStatId()
	{
		Stop();
		StatId = NAME_None;
	}
};

/**
 * This is a utility class for counting the number of cycles during the
 * lifetime of the object. It updates the per thread values for this
 * stat.
 */
class FScopeCycleCounter : public FCycleCounter
{
public:
	/**
	 * Pushes the specified stat onto the hierarchy for this thread. Starts
	 * the timing of the cycles used
	 */
	inline FScopeCycleCounter(TStatId StatId, EStatFlags StatFlags, bool bAlways = false, UE::FSourceLocation SourceLocation = UE::FSourceLocation::Current())
	{
		Start(StatId, StatFlags, bAlways, SourceLocation);
		AutoRTFM::PushOnAbortHandler(this, [this]() { this->Stop(); });
	}

	UE_FORCEINLINE_HINT FScopeCycleCounter(TStatId StatId, bool bAlways = false, UE::FSourceLocation SourceLocation = UE::FSourceLocation::Current())
		: FScopeCycleCounter(StatId, EStatFlags::None, bAlways, SourceLocation)
	{
	}

	/**
	 * Updates the stat with the time spent
	 */
	inline ~FScopeCycleCounter()
	{
		AutoRTFM::PopOnAbortHandler(this);
		Stop();
	}

};

UE_FORCEINLINE_HINT void StatsPrimaryEnableAdd(int32 Value = 1)
{
	FThreadStats::PrimaryEnableAdd(Value);
}
UE_FORCEINLINE_HINT void StatsPrimaryEnableSubtract(int32 Value = 1)
{
	FThreadStats::PrimaryEnableSubtract(Value);
}

class FSimpleScopeSecondsStat
{
public:

	FSimpleScopeSecondsStat(TStatId InStatId, double InScale = 1.0)
		: StartTime(FPlatformTime::Seconds())
		, StatId(InStatId)
		, Scale(InScale)
	{

	}

	virtual ~FSimpleScopeSecondsStat()
	{
		double TotalTime = (FPlatformTime::Seconds() - StartTime) * Scale;
		FThreadStats::AddMessage(StatId.GetName(), EStatOperation::Add, TotalTime);
	}

private:

	double StartTime;
	TStatId StatId;
	double Scale;
};

/** Manages startup messages, usually to update the metadata. */
class FStartupMessages
{
	friend class FStatsThread;

	TArray64<FStatMessage> DelayedMessages;
	FCriticalSection CriticalSection;

public:
	/** Adds a thread metadata. */
	AUTORTFM_DISABLE CORE_API void AddThreadMetadata(const FName InThreadFName, uint32 InThreadID);

	/** Adds a regular metadata. */
	AUTORTFM_DISABLE CORE_API void AddMetadata(FName InStatName, const TCHAR* InStatDesc, const char* InGroupName, const char* InGroupCategory, const TCHAR* InGroupDesc, bool bShouldClearEveryFrame, EStatDataType::Type InStatType, bool bCycleStat, bool bSortByName, FPlatformMemory::EMemoryCounterRegion InMemoryRegion = FPlatformMemory::MCR_Invalid);

	/** Access the singleton. */
	CORE_API static FStartupMessages& Get();
};

/**
* Single interface to control high performance stat disable
*/
class IStatGroupEnableManager
{
public:
	/** Return the singleton, must be called from the main thread. **/
	CORE_API static IStatGroupEnableManager& Get();

	/** virtual destructor. **/
	virtual ~IStatGroupEnableManager() = default;

	/**
	 * Returns a pointer to a bool (valid forever) that determines if this group is active
	 * This should be CACHED. We will get a few calls from different stats and different threads and stuff, but once things are "warmed up", this should NEVER be called.
	 * This function will also register any stats with the StatsTrace system
	 * @param InGroup, group to look up
	 * @param InCategory, the category the group belongs to
	 * @param bDefaultEnable, If this is the first time this group has been set up, this sets the default enable value for this group.
	 * @param bShouldClearEveryFrame, If this is true, this is a memory counter or an accumulator
	 * @return a pointer to a FName (valid forever) that determines if this group is active
	 */
	virtual TStatId GetHighPerformanceEnableForStat(FName StatShortName, const char* InGroup, const char* InCategory, bool bDefaultEnable, bool bShouldClearEveryFrame, EStatDataType::Type InStatType, TCHAR const* InDescription, bool bCycleStat, bool bSortByName, FPlatformMemory::EMemoryCounterRegion MemoryRegion = FPlatformMemory::MCR_Invalid) = 0;

	/**
	 * Enables or disabled a particular group of stats
	 * Disabling a memory group, ever, is usually a bad idea
	 * @param Group, group to look up
	 * @param Enable, this should be true if we want to collect stats for this group
	 */
	virtual void SetHighPerformanceEnableForGroup(FName Group, bool Enable) = 0;

	/**
	 * Enables or disabled all groups of stats
	 * Disabling a memory group, ever, is usually a bad idea. SO if you disable all groups, you will wreck memory stats usually.
	 * @param Enable, this should be true if we want to collect stats for all groups
	 */
	virtual void SetHighPerformanceEnableForAllGroups(bool Enable) = 0;

	/**
	 * Resets all stats to their default collection state, which was set when they were looked up initially
	 */
	virtual void ResetHighPerformanceEnableForAllGroups() = 0;

	/**
	 * Runs a group command
	 * @param Cmd, Command to run
	 */
	virtual void StatGroupEnableManagerCommand(FString const& Cmd) = 0;

	/** Updates memory usage. */
	virtual void UpdateMemoryUsage() = 0;
};


/**
**********************************************************************************
*/
struct FThreadSafeStaticStatBase
{
protected:
	mutable TAtomic<const TStatIdData*> HighPerformanceEnable { nullptr };
	AUTORTFM_OPEN CORE_API const TStatIdData* DoSetup(FName InStatName, const TCHAR* InStatDesc, const char* InGroupName, const char* InGroupCategory, const TCHAR* InGroupDesc, bool bDefaultEnable, bool bShouldClearEveryFrame, EStatDataType::Type InStatType, bool bCycleStat, bool bSortByName, FPlatformMemory::EMemoryCounterRegion InMemoryRegion) const;
};

template<class TStatData, bool TCompiledIn>
struct AUTORTFM_OPEN FThreadSafeStaticStatInner : public FThreadSafeStaticStatBase
{
	inline TStatId GetStatId() const
	{
		const TStatIdData* LocalHighPerformanceEnable = HighPerformanceEnable.Load(EMemoryOrder::Relaxed);
		if (!LocalHighPerformanceEnable)
		{
			LocalHighPerformanceEnable = DoSetup(
				FName(TStatData::GetStatName()),
				TStatData::GetDescription(),
				TStatData::TGroup::GetGroupName(),
				TStatData::TGroup::GetGroupCategory(),
				TStatData::TGroup::GetDescription(),
				TStatData::TGroup::IsDefaultEnabled(),
				TStatData::IsClearEveryFrame(),
				TStatData::GetStatType(),
				TStatData::IsCycleStat(),
				TStatData::TGroup::GetSortByName(),
				TStatData::GetMemoryRegion()
			);
		}
		return TStatId(LocalHighPerformanceEnable);
	}

	UE_FORCEINLINE_HINT FName GetStatFName() const
	{
		return GetStatId().GetName();
	}
};

struct FDynamicStat : public FThreadSafeStaticStatBase
{
	FDynamicStat(
		FName InStatName,
		const TCHAR* InStatDesc,
		const char* InGroupName,
		const char* InGroupCategory,
		const TCHAR* InGroupDesc,
		bool bDefaultEnable,
		bool bShouldClearEveryFrame,
		EStatDataType::Type InStatType,
		bool bCycleStat,
		bool bSortByName,
		FPlatformMemory::EMemoryCounterRegion InMemoryRegion
	)
	{
		HighPerformanceEnable = DoSetup(
			InStatName,
			InStatDesc,
			InGroupName,
			InGroupCategory,
			InGroupDesc,
			bDefaultEnable,
			bShouldClearEveryFrame,
			InStatType,
			bCycleStat,
			bSortByName,
			InMemoryRegion
		);
	}

	UE_FORCEINLINE_HINT TStatId GetStatId() const
	{
		return TStatId(HighPerformanceEnable.Load(EMemoryOrder::Relaxed));
	}

	UE_FORCEINLINE_HINT FName GetStatFName() const
	{
		return GetStatId().GetName();
	}
};

template<class TStatData>
struct FThreadSafeStaticStatInner<TStatData, false>
{
	UE_FORCEINLINE_HINT TStatId GetStatId()
	{
		return TStatId();
	}
	UE_FORCEINLINE_HINT FName GetStatFName() const
	{
		return FName();
	}
};

template<class TStatData>
struct FThreadSafeStaticStat : public FThreadSafeStaticStatInner<TStatData, TStatData::TGroup::CompileTimeEnable>
{
	FThreadSafeStaticStat()
	{
		//This call will result in registering the Group if it's compile time enabled. 
		//It fixes a bug when a StatGroup only has counters that are using the INC_\DEC_ macros. 
		//Those macros are guarded for the stats collection to be active which prevented the registration of the stat group.
		//It was not possible to activate the stat group unless another was already active.
		//Most groups are registered when a FScopeCycleCounter is declared as GetStatId is called as the constructor parameter.
		FThreadSafeStaticStatInner<TStatData, TStatData::TGroup::CompileTimeEnable>::GetStatId(); //-V530
	}
};

#else // STATS

inline void StatsPrimaryEnableAdd(int32 Value = 1)
{
}
inline void StatsPrimaryEnableSubtract(int32 Value = 1)
{
}

#endif // STATS
