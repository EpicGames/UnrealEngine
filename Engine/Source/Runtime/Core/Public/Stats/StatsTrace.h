// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Build.h"
#include "Trace/Config.h"

#if !defined(STATSTRACE_ENABLED)
#if UE_TRACE_ENABLED && STATS && !UE_BUILD_SHIPPING
#define STATSTRACE_ENABLED 1
#else
#define STATSTRACE_ENABLED 0
#endif
#endif

#if STATSTRACE_ENABLED

class FName;

struct FStatsTrace
{
	CORE_API static void DeclareStat(const FName& Stat, const ANSICHAR* Name, const TCHAR* Description, const ANSICHAR* Group, bool IsFloatingPoint, bool IsMemory, bool ShouldClearEveryFrame);
	CORE_API static void Increment(const FName& Stat);
	CORE_API static void Decrement(const FName& Stat);
	CORE_API static void Add(const FName& Stat, int64 Amount);
	CORE_API static void Add(const FName& Stat, double Amount);
	CORE_API static void Set(const FName& Stat, int64 Value);
	CORE_API static void Set(const FName& Stat, double Value);
};

#define TRACE_STAT_INCREMENT(Stat) \
	FStatsTrace::Increment(Stat);

#define TRACE_STAT_DECREMENT(Stat) \
	FStatsTrace::Decrement(Stat);

#define TRACE_STAT_ADD(Stat, Amount) \
	FStatsTrace::Add(Stat, Amount);

#define TRACE_STAT_SET(Stat, Value) \
	FStatsTrace::Set(Stat, Value);

#else

#define TRACE_STAT_INCREMENT(...)
#define TRACE_STAT_DECREMENT(...)
#define TRACE_STAT_ADD(...)
#define TRACE_STAT_SET(...)

#endif
