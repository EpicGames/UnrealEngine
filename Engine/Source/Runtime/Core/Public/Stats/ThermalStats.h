// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Stats/Stats.h"
#include "ProfilingDebugging/CsvProfiler.h"

DECLARE_STATS_GROUP(TEXT("Thermal Stats"), STATGROUP_Thermal, STATCAT_Advanced);

// Generic thermal stats
// Note that readings represented by generic thermal stats are similar but not exactly the same across platforms or even across device models.
// If no similar reading exists the generic stat should be left unsupported.

// DeviceTemperature stat for the device.  
// This is meant to represent the 'main' temperature reading related to throttling or shutdown on a device.
// However it might be a cpu measurement, it might be a battery measurement, etc.  The temperatures at which throttling occurs might vary between devices.
// Be cautious about comparing this value across different devices models even within the same platform.
// This may be duplicated in a platform specific stat.
// Look at the platform specific code that fills it in and refer to platform documentation for more detail.
// It is possible that this value will only be available on certain non-shipping builds on some platforms.
// -1 or 0 will indicate that no value is being collected (though theoretically they could also be actual temperatures).
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Device Temperature"), STAT_DeviceTemperature, STATGROUP_Thermal, CORE_API);

// Qualitative thermal state of the device.
// See EDeviceThermalState for more information.
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Thermal State"), STAT_ThermalState, STATGROUP_Thermal, CORE_API);

CSV_DECLARE_CATEGORY_EXTERN(Thermal);
CSV_DECLARE_STAT_EXTERN(Thermal, DeviceTemperature);
CSV_DECLARE_STAT_EXTERN(Thermal, ThermalState);

namespace UE
{
	namespace ThermalStats
	{
		// One option is to call the following function to pull the stats using FPlatformMisc accessors, 
		// then push them out to stats.
		void GetAndPushStats();

		// Another option is to push the stats defined above directly in platform code.
		// If you do that you should still implement FPlatformMisc::GetDeviceTemperature(), etc. There are other ways they are output.
	}
}