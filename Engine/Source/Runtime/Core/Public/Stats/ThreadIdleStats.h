// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformTime.h"
#include "HAL/ThreadSingleton.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

namespace UE::Stats
{
	class FThreadIdleStats;
}

UE_DECLARE_THREAD_SINGLETON_TLS(UE::Stats::FThreadIdleStats, CORE_API)

namespace UE::Stats
{

	/**
	* This is thread-private information about the thread idle stats, which we always collect, even in final builds
	*/
	class FThreadIdleStats : public TThreadSingleton<FThreadIdleStats>
	{
		friend class TThreadSingleton<FThreadIdleStats>;

		FThreadIdleStats()
			: Waits(0)
			, WaitsCriticalPath(0)
			, IsCriticalPathCounter(1)
			, bInIdleScope(false)
		{}

	public:

		/** Total cycles we waited for sleep or event. **/
		uint32 Waits;

		/** Total cycles we waited for sleep or event on the critical path. **/
		uint32 WaitsCriticalPath;

		int IsCriticalPathCounter;
		bool bInIdleScope;

		static void BeginCriticalPath()
		{
			FThreadIdleStats::Get().IsCriticalPathCounter++;
		}

		static void EndCriticalPath()
		{
			FThreadIdleStats::Get().IsCriticalPathCounter--;
		}

		struct FScopeNonCriticalPath
		{
			FScopeNonCriticalPath()
			{
				FThreadIdleStats::Get().IsCriticalPathCounter--;
			}
			~FScopeNonCriticalPath()
			{
				FThreadIdleStats::Get().IsCriticalPathCounter++;
			}
		};

		bool IsCriticalPath() const
		{
			return IsCriticalPathCounter > 0;
		}

		void Reset()
		{
			Waits = 0;
			WaitsCriticalPath = 0;
			IsCriticalPathCounter = 1;
		}



		struct FScopeIdle
		{
#if defined(DISABLE_THREAD_IDLE_STATS) && DISABLE_THREAD_IDLE_STATS
			FScopeIdle(bool bInIgnore = false)
			{}
#else
			/** Starting cycle counter. */
			const uint32 Start;

			/** If true, we ignore this thread idle stats. */
			const bool bIgnore;

#if CPUPROFILERTRACE_ENABLED
			FCpuProfilerTrace::FEventScope TraceEventScope;
#endif

			CORE_API FScopeIdle(bool bInIgnore = false);

			~FScopeIdle()
			{
				if (!bIgnore)
				{
					FThreadIdleStats& IdleStats = FThreadIdleStats::Get();
					uint32 CyclesElapsed = FPlatformTime::Cycles() - Start;
					IdleStats.Waits += CyclesElapsed;

					if (IdleStats.IsCriticalPath())
					{
						IdleStats.WaitsCriticalPath += CyclesElapsed;
					}

					IdleStats.bInIdleScope = false;
				}
			}
#endif
		};
	};
} // namespace UE::Stats
