// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "CoreTypes.h"

namespace UE::String
{

/**
 * Convert an array of bytes to a non-null-terminated string of hex digits.
 *
 * @param Bytes Array of bytes to convert.
 * @param OutHex [out] Array of at least 2*Bytes.Len() output characters.
 */
CORE_API void BytesToHex(TConstArrayView<uint8> Bytes, ANSICHAR* OutHex);
CORE_API void BytesToHex(TConstArrayView<uint8> Bytes, WIDECHAR* OutHex);
CORE_API void BytesToHex(TConstArrayView<uint8> Bytes, UTF8CHAR* OutHex);
CORE_API void BytesToHexLower(TConstArrayView<uint8> Bytes, ANSICHAR* OutHex);
CORE_API void BytesToHexLower(TConstArrayView<uint8> Bytes, WIDECHAR* OutHex);
CORE_API void BytesToHexLower(TConstArrayView<uint8> Bytes, UTF8CHAR* OutHex);

/**
 * Append an array of bytes to a string builder as hex digits.
 *
 * @param Bytes Array of bytes to convert.
 * @param Builder Builder to append the converted string to.
 */
CORE_API void BytesToHex(TConstArrayView<uint8> Bytes, FAnsiStringBuilderBase& Builder);
CORE_API void BytesToHex(TConstArrayView<uint8> Bytes, FWideStringBuilderBase& Builder);
CORE_API void BytesToHex(TConstArrayView<uint8> Bytes, FUtf8StringBuilderBase& Builder);
CORE_API void BytesToHexLower(TConstArrayView<uint8> Bytes, FAnsiStringBuilderBase& Builder);
CORE_API void BytesToHexLower(TConstArrayView<uint8> Bytes, FWideStringBuilderBase& Builder);
CORE_API void BytesToHexLower(TConstArrayView<uint8> Bytes, FUtf8StringBuilderBase& Builder);

} // UE::String
