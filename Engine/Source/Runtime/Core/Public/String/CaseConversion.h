// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "CoreTypes.h"

namespace UE::String::Private
{

template <typename CharType>
struct TUpperCase { TStringView<CharType> Input; };

template <typename CharType>
struct TLowerCase { TStringView<CharType> Input; };

template <typename CharType>
struct TPascalCase { TStringView<CharType> Input; };

} // UE::String::Private

namespace UE::String
{

/**
 * Convert the string to uppercase and append to the string builder.
 * 
 * @note Only ASCII characters will be converted, similar to CRT to[w]upper() with the standard C locale.
 * @param Input   The string to convert to uppercase.
 * @param Output  The string builder to append the converted string to.
 * 
 * @code
 *	UpperCaseTo(TEXTVIEW("example"), Output); // Output now contains "EXAMPLE"
 * @endcode
 */
CORE_API void UpperCaseTo(FAnsiStringView Input, FAnsiStringBuilderBase& Output);
CORE_API void UpperCaseTo(FUtf8StringView Input, FUtf8StringBuilderBase& Output);
CORE_API void UpperCaseTo(FWideStringView Input, FWideStringBuilderBase& Output);

/**
 * Convert the string to lowercase and append to the string builder.
 *
 * @note Only ASCII characters will be converted, similar to CRT to[w]upper() with the standard C locale.
 * @param Input   The string to convert to lowercase.
 * @param Output  The string builder to append the converted string to.
 * 
 * @code
 *	LowerCaseTo(TEXTVIEW("EXAMPLE"), Output); // Output now contains "example"
 * @endcode
 */
CORE_API void LowerCaseTo(FAnsiStringView Input, FAnsiStringBuilderBase& Output);
CORE_API void LowerCaseTo(FUtf8StringView Input, FUtf8StringBuilderBase& Output);
CORE_API void LowerCaseTo(FWideStringView Input, FWideStringBuilderBase& Output);

/**
 * Convert the string to PascalCase and append to the string builder.
 *
 * @note Only ASCII characters will be converted.
 * @param Input   The string to convert to PascalCase.
 * @param Output  The string builder to append the converted string to.
 *
 * @code
 *	PascalCaseTo(TEXTVIEW("EXAMPLE TEXT"), Output); // Output now contains "ExampleText"
 * @endcode
 */
CORE_API void PascalCaseTo(FStringView Input, FStringBuilderBase& Output);

/**
 * Convert the string to uppercase when appended to a string builder.
 *
 * @note Only ASCII characters will be converted, similar to CRT to[w]upper() with the standard C locale.
 * @param Input   The string to convert to uppercase.
 * @return An anonymous object to append to a string builder.
 * 
 * @code
 *	Builder << String::UpperCase(TEXTVIEW("example")); // Builder now contains "EXAMPLE"
 * @endcode
 */
inline Private::TUpperCase<ANSICHAR> UpperCase(FAnsiStringView Input)
{
	return {Input};
}
inline Private::TUpperCase<UTF8CHAR> UpperCase(FUtf8StringView Input)
{
	return {Input};
}
inline Private::TUpperCase<WIDECHAR> UpperCase(FWideStringView Input)
{
	return {Input};
}

/**
 * Convert the string to lowercase when appended to a string builder.
 *
 * @note Only ASCII characters will be converted, similar to CRT to[w]upper() with the standard C locale.
 * @param Input   The string to convert to lowercase.
 * @return An anonymous object to append to a string builder.
 * 
 * @code
 *	Builder << String::LowerCase(TEXTVIEW("EXAMPLE")); // Builder now contains "example"
 * @endcode
 */
inline Private::TLowerCase<ANSICHAR> LowerCase(FAnsiStringView Input)
{
	return {Input};
}
inline Private::TLowerCase<UTF8CHAR> LowerCase(FUtf8StringView Input)
{
	return {Input};
}
inline Private::TLowerCase<WIDECHAR> LowerCase(FWideStringView Input)
{
	return {Input};
}

/**
 * Convert the string to PascalCase when appended to a string builder.
 *
 * @note Only ASCII characters will be converted.
 * @param Input   The string to convert to PascalCase.
 * @return An anonymous object to append to a string builder.
 *
 * @code
 *	Builder << String::PascalCase(TEXTVIEW("EXAMPLE TEXT")); // Builder now contains "ExampleText"
 * @endcode
 */
inline Private::TPascalCase<TCHAR> PascalCase(FStringView Input)
{
	return {Input};
}

} // UE::String

namespace UE::String::Private
{

template <typename CharType>
inline TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const TUpperCase<CharType>& Adapter)
{
	UpperCaseTo(Adapter.Input, Builder);
	return Builder;
}

template <typename CharType>
inline TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const TLowerCase<CharType>& Adapter)
{
	LowerCaseTo(Adapter.Input, Builder);
	return Builder;
}

template <typename CharType>
inline TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, const TPascalCase<CharType>& Adapter)
{
	PascalCaseTo(Adapter.Input, Builder);
	return Builder;
}

} // UE::String::Private
