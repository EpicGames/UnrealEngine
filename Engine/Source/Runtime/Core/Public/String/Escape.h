// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "CoreTypes.h"

namespace UE::String::Private
{

struct FEscape { FStringView Input; };
struct FQuoteEscape { FStringView Input; };

} // UE::String::Private

namespace UE::String
{

/**
 * Escape the string into the string builder.
 *
 * Example: Escape("Tab is \t and \" is a quote.", Output) -> "Tab is \\t and \\\" is a quote."
 */
CORE_API void EscapeTo(FStringView Input, FStringBuilderBase& Output);

/**
 * Escape the string when appended to a string builder.
 *
 * Example: Builder << String::Escape(Value);
 */
inline Private::FEscape Escape(FStringView Input) { return {Input}; }

/**
 * Quote and escape the string into the string builder.
 *
 * Example: QuoteEscape("Tab is \t and \" is a quote.", Output) -> "\"Tab is \\t and \\\" is a quote.\""
 */
CORE_API void QuoteEscapeTo(FStringView Input, FStringBuilderBase& Output);

/**
 * Quote and escape the string when appended to a string builder.
 *
 * Example: Builder << String::QuoteEscape(Value);
 */
inline Private::FQuoteEscape QuoteEscape(FStringView Input) { return {Input}; }

} // UE::String

namespace UE::String::Private
{

inline FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FEscape& Adapter)
{
	EscapeTo(Adapter.Input, Builder);
	return Builder;
}

inline FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FQuoteEscape& Adapter)
{
	QuoteEscapeTo(Adapter.Input, Builder);
	return Builder;
}

} // UE::String::Private
