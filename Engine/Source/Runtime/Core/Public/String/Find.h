// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "CoreTypes.h"
#include "Misc/CString.h"

namespace UE::String
{

/**
 * Search the view for the first occurrence of the search string.
 *
 * @param View The string to search within.
 * @param Search The string to search for.
 * @param SearchCase Whether the comparison should ignore case.
 * @return The position at which the search string was found, or INDEX_NONE if not found.
 */
CORE_API int32 FindFirst(FUtf8StringView View, FUtf8StringView Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);
CORE_API int32 FindFirst(FWideStringView View, FWideStringView Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);

/**
 * Search the view for the last occurrence of the search string.
 *
 * @param View The string to search within.
 * @param Search The string to search for.
 * @param SearchCase Whether the comparison should ignore case.
 * @return The position at which the search string was found, or INDEX_NONE if not found.
 */
CORE_API int32 FindLast(FUtf8StringView View, FUtf8StringView Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);
CORE_API int32 FindLast(FWideStringView View, FWideStringView Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);

/**
 * Search the view for the first occurrence of any search string.
 *
 * @param View The string to search within.
 * @param Search The strings to search for.
 * @param SearchCase Whether the comparison should ignore case.
 * @param OutMatchIndex Receives the index of the search string that matched.
 * @return The position at which any search string was found, or INDEX_NONE if not found.
 */
CORE_API int32 FindFirstOfAny(FAnsiStringView View, TConstArrayView<FAnsiStringView> Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API int32 FindFirstOfAny(FUtf8StringView View, TConstArrayView<FUtf8StringView> Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API int32 FindFirstOfAny(FWideStringView View, TConstArrayView<FWideStringView> Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);

/**
 * Search the view for the last occurrence of any search string.
 *
 * @param View The string to search within.
 * @param Search The strings to search for.
 * @param SearchCase Whether the comparison should ignore case.
 * @param OutMatchIndex Receives the index of the search string that matched.
 * @return The position at which any search string was found, or INDEX_NONE if not found.
 */
CORE_API int32 FindLastOfAny(FAnsiStringView View, TConstArrayView<FAnsiStringView> Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API int32 FindLastOfAny(FUtf8StringView View, TConstArrayView<FUtf8StringView> Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API int32 FindLastOfAny(FWideStringView View, TConstArrayView<FWideStringView> Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);

/**
 * Search the view for the first occurrence of the search character.
 *
 * @param View The string to search within.
 * @param Search The character to search for.
 * @param SearchCase Whether the comparison should ignore case.
 * @return The position at which the search character was found, or INDEX_NONE if not found.
 */
CORE_API int32 FindFirstChar(FUtf8StringView View, UTF8CHAR Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);
CORE_API int32 FindFirstChar(FWideStringView View, WIDECHAR Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);

/**
 * Search the view for the last occurrence of the search character.
 *
 * @param View The string to search within.
 * @param Search The character to search for.
 * @param SearchCase Whether the comparison should ignore case.
 * @return The position at which the search character was found, or INDEX_NONE if not found.
 */
CORE_API int32 FindLastChar(FUtf8StringView View, UTF8CHAR Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);
CORE_API int32 FindLastChar(FWideStringView View, WIDECHAR Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);

/**
 * Search the view for the first occurrence of any search character.
 *
 * @param View The string to search within.
 * @param Search The characters to search for.
 * @param SearchCase Whether the comparison should ignore case.
 * @param OutMatchIndex Receives the index of the search character that matched.
 * @return The position at which any search character was found, or INDEX_NONE if not found.
 */
CORE_API int32 FindFirstOfAnyChar(FUtf8StringView View, std::initializer_list<UTF8CHAR> Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API int32 FindFirstOfAnyChar(FUtf8StringView View, TConstArrayView<ANSICHAR> Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API int32 FindFirstOfAnyChar(FUtf8StringView View, TConstArrayView<UTF8CHAR> Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API int32 FindFirstOfAnyChar(FWideStringView View, TConstArrayView<WIDECHAR> Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);

/**
 * Search the view for the last occurrence of any search character.
 *
 * @param View The string to search within.
 * @param Search The characters to search for.
 * @param SearchCase Whether the comparison should ignore case.
 * @param OutMatchIndex Receives the index of the search character that matched.
 * @return The position at which any search character was found, or INDEX_NONE if not found.
 */
CORE_API int32 FindLastOfAnyChar(FUtf8StringView View, std::initializer_list<UTF8CHAR> Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API int32 FindLastOfAnyChar(FUtf8StringView View, TConstArrayView<ANSICHAR> Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API int32 FindLastOfAnyChar(FUtf8StringView View, TConstArrayView<UTF8CHAR> Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API int32 FindLastOfAnyChar(FWideStringView View, TConstArrayView<WIDECHAR> Search, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);

} // UE::String
