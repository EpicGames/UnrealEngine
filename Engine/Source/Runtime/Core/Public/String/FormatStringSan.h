// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <type_traits>
#include "CoreTypes.h"
#include "Templates/Requires.h"
#include "Templates/Identity.h"
#include "Templates/IsValidVariadicFunctionArg.h"
#include "Traits/IsCharType.h"
#include "Traits/IsTEnumAsByte.h"
#include "Traits/IsTString.h"
#include "Containers/ContainersFwd.h"

#define UE_CHECK_FORMAT_STRING(Fmt, ...) \
	do { \
		namespace UCFS = ::UE::Core::Private::FormatStringSan; \
		using UCFS_FChecker = decltype(UCFS::GetFmtArgCheckerType(Fmt, ##__VA_ARGS__)); \
		constexpr UCFS::FResult UCFS_Result = UCFS_FChecker::Check(false, 0, Fmt); \
		(UCFS::AssertFormatStatus<UCFS_Result.Status, UCFS::TAtArgPos<UCFS_Result.ArgPos>>()); \
	} while(false);

#define UE_CHECK_FORMAT_STRING_ERR(Err, Fmt, ...) \
	(decltype(::UE::Core::Private::FormatStringSan::GetFmtArgCheckerType(Fmt, ##__VA_ARGS__))::Check(false, 0, Fmt).Status == Err)


#if UE_VALIDATE_FORMAT_STRINGS
	#define UE_VALIDATE_FORMAT_STRING UE_CHECK_FORMAT_STRING
#else
	#define UE_VALIDATE_FORMAT_STRING(Format, ...)
#endif

/// implementation
namespace UE::Core::Private
{
	namespace FormatStringSan
	{
		// Returns true when the type is a const char*, const TCHAR*, const char[] or const TCHAR[]
		template<typename T>
		inline constexpr bool bIsAConstString =
			   !(std::is_convertible_v<T, char*> || std::is_convertible_v<T, TCHAR*>)
			&& (std::is_convertible_v<T, const char*> || std::is_convertible_v<T, const TCHAR*>);

		enum class EFormatStringSanStatus
		{
			Ok,

			#define X(Name, Desc) Name,
			#include "FormatStringSanErrors.inl"
			#undef X
		};

		template <EFormatStringSanStatus Err, typename T>
		constexpr void AssertFormatStatus()
		{
			switch (Err)
			{
				case EFormatStringSanStatus::Ok: break;

				#define X(Name, Desc) \
				case EFormatStringSanStatus::Name: static_assert(Err != EFormatStringSanStatus::Name, Desc); break;
				#include "FormatStringSanErrors.inl"
				#undef X
			}
		}

		template <int N>
		struct TAtArgPos {};

		struct FResult
		{
			EFormatStringSanStatus Status{};
			int ArgPos{};
		};

		template <typename T>
		inline constexpr bool TIsFloatOrDouble_V = std::is_same_v<float, T> || std::is_same_v<double, T>;

		template <typename T>
		inline constexpr bool TIsIntegralEnum_V = std::is_enum_v<T> || TIsTEnumAsByte_V<T>;

		template <typename CharType, typename...>
		struct TCheckFormatString;

		template <typename CharType, typename... Ts>
		constexpr TCheckFormatString<CharType, std::decay_t<Ts>...> GetFmtArgCheckerType(const CharType*, Ts...);

		template <typename CharType>
		constexpr bool CharIsDigit(CharType Char)
		{
			return Char >= (CharType)'0' && Char <= (CharType)'9';
		}

		template <typename CharType>
		constexpr const CharType* SkipInteger(const CharType* Fmt)
		{
			while (CharIsDigit(*Fmt))
			{
				++Fmt;
			}
			return Fmt;
		}

		template <typename CharType>
		constexpr bool CharIsIntegerFormatSpecifier(CharType Char)
		{
			switch (Char)
			{
			case (CharType)'i': case (CharType)'d': case (CharType)'u': case (CharType)'X': case (CharType)'x':
				return true;
			default:
				return false;
			}
		}

		template <typename CharType, typename Arg, typename... Args>
		struct TCheckFormatString<CharType, Arg, Args...>
		{
			static constexpr FResult HandleDynamicLengthSpecifier(int CurArgPos, const CharType* Fmt)
			{
				if constexpr (!(std::is_integral_v<Arg> || TIsIntegralEnum_V<Arg>))
				{
					return {EFormatStringSanStatus::DynamicLengthSpecNeedsIntegerArg, CurArgPos};
				}
				else
				{
					return TCheckFormatString<CharType, Args...>::Check(true, CurArgPos + 1, Fmt + 1);
				}
			}

			static constexpr FResult Check(bool bInsideFormatSpec, int CurArgPos, const CharType* Fmt)
			{
				using ArgPointeeType = std::remove_pointer_t<Arg>;

				if (!bInsideFormatSpec)
				{
					while (*Fmt != (CharType)'\0' && *Fmt != (CharType)'%')
					{
						++Fmt;
					}

					if (Fmt[0] == (CharType)'\0')
					{
						return { EFormatStringSanStatus::NotEnoughSpecifiers, CurArgPos };
					}

					if (*Fmt == (CharType)'%')
					{
						++Fmt;
					}
				}

				if (*Fmt == (CharType)'\0')
				{
					if (Fmt[-1] == (CharType)'%' || bInsideFormatSpec) // nb: Fmt[-1] is safe here because zero-length fmt strings are addressed above
					{
						return {EFormatStringSanStatus::IncompleteFormatSpecifierOrUnescapedPercent, CurArgPos};
					}
					else
					{
						return {EFormatStringSanStatus::Ok, 0};
					}
				}

				while (*Fmt == (CharType)'+' || *Fmt == (CharType)'#' || *Fmt == (CharType)' ' || *Fmt == (CharType)'0')
				{
					++Fmt;
				}

				if (*Fmt == (CharType)'-')
				{
					++Fmt;
				}
				if (*Fmt == (CharType)'*')
				{
					return HandleDynamicLengthSpecifier(CurArgPos, Fmt);
				}
				else if (CharIsDigit(*Fmt))
				{
					Fmt = SkipInteger(Fmt);
				}

				if (*Fmt == (CharType)'.')
				{
					++Fmt;
				}
				if (*Fmt == (CharType)'*')
				{
					return HandleDynamicLengthSpecifier(CurArgPos, Fmt);
				}
				else if (CharIsDigit(*Fmt))
				{
					Fmt = SkipInteger(Fmt);
				}

				if (Fmt[0] == (CharType)'l' && Fmt[1] == (CharType)'s')
				{
					if constexpr (TIsTString_V<Arg>)
					{
						return {EFormatStringSanStatus::LSNeedsDereferencedWideString, CurArgPos};
					}
					else if constexpr (TIsCharType_V<Arg>)
					{
						return {EFormatStringSanStatus::LSNeedsPtrButGotChar, CurArgPos};
					}
					else if constexpr (!std::is_pointer_v<Arg> || !TIsCharType_V<ArgPointeeType>)
					{
						return {EFormatStringSanStatus::LSNeedsWideCharPtrArg, CurArgPos};
					}
					else if constexpr (sizeof(ArgPointeeType) != sizeof(WIDECHAR))
					{
						return {(sizeof(CharType) == 1) ? EFormatStringSanStatus::LSNeedsWideCharPtrArgButGotNarrowOnNarrowString : EFormatStringSanStatus::LSNeedsWideCharPtrArgButGotNarrowOnWideString, CurArgPos};
					}
					else
					{
						return TCheckFormatString<CharType, Args...>::Check(false, CurArgPos + 1, Fmt + 2);
					}
				}
				else if (Fmt[0] == (CharType)'h' && Fmt[1] == (CharType)'s')
				{
					if constexpr (TIsTString_V<Arg>)
					{
						return {EFormatStringSanStatus::HSNeedsDereferencedNarrowString, CurArgPos};
					}
					else if constexpr (TIsCharType_V<Arg>)
					{
						return {EFormatStringSanStatus::HSNeedsPtrButGotChar, CurArgPos};
					}
					else if constexpr (!std::is_pointer_v<Arg> || !TIsCharType_V<ArgPointeeType>)
					{
						return {EFormatStringSanStatus::HSNeedsNarrowCharPtrArg, CurArgPos};
					}
					else if constexpr (sizeof(ArgPointeeType) != sizeof(ANSICHAR))
					{
						return {(sizeof(CharType) == 1) ? EFormatStringSanStatus::HSNeedsNarrowCharPtrArgButGotWideOnNarrowString : EFormatStringSanStatus::HSNeedsNarrowCharPtrArgButGotWideOnWideString, CurArgPos};
					}
					else
					{
						return TCheckFormatString<CharType, Args...>::Check(false, CurArgPos + 1, Fmt + 2);
					}
				}
				else if (Fmt[0] == (CharType)'s')
				{
					if constexpr (TIsTString_V<Arg>)
					{
						return {(sizeof(CharType) == 1) ? EFormatStringSanStatus::SNeedsDereferencedNarrowString : EFormatStringSanStatus::SNeedsDereferencedWideString, CurArgPos};
					}
					else if constexpr (TIsCharType_V<Arg>)
					{
						return {EFormatStringSanStatus::SNeedsPtrButGotChar, CurArgPos};
					}
					else if constexpr (!std::is_pointer_v<Arg> || !TIsCharType_V<ArgPointeeType>)
					{
						return {(sizeof(CharType) == 1) ? EFormatStringSanStatus::SNeedsNarrowCharPtrArg : EFormatStringSanStatus::SNeedsWideCharPtrArg, CurArgPos};
					}
					else if constexpr (sizeof(ArgPointeeType) != sizeof(CharType))
					{
						return {(sizeof(CharType) == 1) ? EFormatStringSanStatus::SNeedsNarrowCharPtrArgButGotWide : EFormatStringSanStatus::SNeedsWideCharPtrArgButGotNarrow, CurArgPos};
					}
					else
					{
						return TCheckFormatString<CharType, Args...>::Check(false, CurArgPos + 1, Fmt + 1);
					}
				}
				else if (Fmt[0] == (CharType)'S')
				{
					if constexpr (TIsTString_V<Arg>)
					{
						return {(sizeof(CharType) == 1) ? EFormatStringSanStatus::CapitalSNeedsDereferencedWideString : EFormatStringSanStatus::CapitalSNeedsDereferencedNarrowString, CurArgPos};
					}
					else if constexpr (TIsCharType_V<Arg>)
					{
						return {EFormatStringSanStatus::CapitalSNeedsPtrButGotChar, CurArgPos};
					}
					else if constexpr (!std::is_pointer_v<Arg> || !TIsCharType_V<ArgPointeeType>)
					{
						return {(sizeof(CharType) == 1) ? EFormatStringSanStatus::CapitalSNeedsWideCharPtrArg : EFormatStringSanStatus::CapitalSNeedsNarrowCharPtrArg, CurArgPos};
					}
					else if constexpr (sizeof(ArgPointeeType) == sizeof(CharType))
					{
						return {(sizeof(CharType) == 1) ? EFormatStringSanStatus::CapitalSNeedsWideCharPtrArgButGotNarrow : EFormatStringSanStatus::CapitalSNeedsNarrowCharPtrArgButGotWide, CurArgPos};
					}
					else
					{
						return TCheckFormatString<CharType, Args...>::Check(false, CurArgPos + 1, Fmt + 1);
					}
				}

				switch (Fmt[0])
				{
				case (CharType)'%':
					return TCheckFormatString<CharType, Arg, Args...>::Check(false, CurArgPos, Fmt + 1);
				case (CharType)'c':
					if constexpr (!std::is_same_v<Arg, UTF8CHAR> && !(std::is_integral_v<Arg> && sizeof(Arg) <= sizeof(int)))
					{
						return {(sizeof(CharType) == 1) ? EFormatStringSanStatus::CNeedsCharArgOnNarrowString : EFormatStringSanStatus::CNeedsCharArgOnWideString, CurArgPos};
					}
					else
					{
						return TCheckFormatString<CharType, Args...>::Check(false, CurArgPos + 1, Fmt + 1);
					}
				case (CharType)'d':
				case (CharType)'i':
				case (CharType)'X':
				case (CharType)'x':
				case (CharType)'u':
					if constexpr (std::is_pointer_v<Arg> && TIsCharType_V<ArgPointeeType>)
					{
						return {EFormatStringSanStatus::DNeedsIntegerArg, CurArgPos};
					}
					else if constexpr (!(TIsIntegralEnum_V<Arg> || std::is_integral_v<Arg> || std::is_pointer_v<Arg>))
					{
						return {EFormatStringSanStatus::DNeedsIntegerArg, CurArgPos};
					}
					else
					{
						return TCheckFormatString<CharType, Args...>::Check(false, CurArgPos + 1, Fmt + 1);
					}
				case (CharType)'z':
					if (!CharIsIntegerFormatSpecifier(Fmt[1]))
					{
						return {EFormatStringSanStatus::ZNeedsIntegerSpec, CurArgPos};
					}
					else if constexpr (!(TIsIntegralEnum_V<Arg> || std::is_integral_v<Arg>))
					{
						return {EFormatStringSanStatus::ZNeedsIntegerArg, CurArgPos};
					}
					else
					{
						return TCheckFormatString<CharType, Args...>::Check(false, CurArgPos + 1, Fmt + 1);
					}
				case (CharType)'p':
					if constexpr (!std::is_pointer_v<Arg>)
					{
						return {EFormatStringSanStatus::PNeedsPointerArg, CurArgPos};
					}
					else
					{
						return TCheckFormatString<CharType, Args...>::Check(false, CurArgPos + 1, Fmt + 1);
					}
				case (CharType)'I':
					if (!(Fmt[1] == (CharType)'6' && Fmt[2] == (CharType)'4'))
					{
						return {EFormatStringSanStatus::I64BadSpec, CurArgPos};
					}
					else if (!CharIsIntegerFormatSpecifier(Fmt[3]))
					{
						return {EFormatStringSanStatus::I64BadSpec, CurArgPos};
					}
					else if constexpr (!(TIsIntegralEnum_V<Arg> || std::is_integral_v<Arg>))
					{
						return {EFormatStringSanStatus::I64NeedsIntegerArg, CurArgPos};
					}
					else
					{
						return TCheckFormatString<CharType, Args...>::Check(false, CurArgPos + 1, Fmt + 4);
					}

				case (CharType)'l':
					if (CharIsIntegerFormatSpecifier(Fmt[1]))
					{
						if constexpr (!(TIsIntegralEnum_V<Arg> || std::is_integral_v<Arg>))
						{
							return {EFormatStringSanStatus::LNeedsIntegerArg, CurArgPos};
						}
						else
						{
							return TCheckFormatString<CharType, Args...>::Check(false, CurArgPos + 1, Fmt + 2);
						}
					}
					else if (Fmt[1] == (CharType)'f')
					{
						if constexpr (!TIsFloatOrDouble_V<Arg>)
						{
							return {EFormatStringSanStatus::FNeedsFloatOrDoubleArg, CurArgPos};
						}
						else
						{
							return TCheckFormatString<CharType, Args...>::Check(false, CurArgPos + 1, Fmt + 2);
						}
					}
					else if (Fmt[1] != (CharType)'l')
					{
						return {EFormatStringSanStatus::InvalidFormatSpec, CurArgPos};
					}
					else if (!CharIsIntegerFormatSpecifier(Fmt[2]))
					{
						return {EFormatStringSanStatus::LLNeedsIntegerSpec, CurArgPos};
					}
					else if constexpr (!(TIsIntegralEnum_V<Arg> || std::is_integral_v<Arg> || std::is_pointer_v<Arg>))
					{
						return {EFormatStringSanStatus::LLNeedsIntegerArg, CurArgPos};
					}
					else
					{
						return TCheckFormatString<CharType, Args...>::Check(false, CurArgPos + 1, Fmt + 3);
					}

				case (CharType)'h':
					if (CharIsIntegerFormatSpecifier(Fmt[1]))
					{
						if constexpr (!(TIsIntegralEnum_V<Arg> || std::is_integral_v<Arg>))
						{
							return {EFormatStringSanStatus::HNeedsIntegerArg, CurArgPos};
						}
						return TCheckFormatString<CharType, Args...>::Check(false, CurArgPos + 1, Fmt + 2);
					}
					else if (Fmt[1] != (CharType)'h')
					{
						return {EFormatStringSanStatus::InvalidFormatSpec, CurArgPos};
					}
					else if (!CharIsIntegerFormatSpecifier(Fmt[2]))
					{
						return {EFormatStringSanStatus::HHNeedsIntegerSpec, CurArgPos};
					}
					else if constexpr (!(TIsIntegralEnum_V<Arg> || std::is_integral_v<Arg>))
					{
						return {EFormatStringSanStatus::HHNeedsIntegerArg, CurArgPos};
					}
					else
					{
						return TCheckFormatString<CharType, Args...>::Check(false, CurArgPos + 1, Fmt + 3);
					}

				case (CharType)'f':
				case (CharType)'e':
				case (CharType)'g':
					if constexpr (!TIsFloatOrDouble_V<Arg>)
					{
						return {EFormatStringSanStatus::FNeedsFloatOrDoubleArg, CurArgPos};
					}
					else
					{
						return TCheckFormatString<CharType, Args...>::Check(false, CurArgPos + 1, Fmt + 1);
					}

				case (CharType)' ':
					return {EFormatStringSanStatus::IncompleteFormatSpecifierOrUnescapedPercent, CurArgPos};

				default:
					return {EFormatStringSanStatus::InvalidFormatSpec, CurArgPos};
				}
			}
		};

		template <typename CharType>
		struct TCheckFormatString<CharType>
		{
			static constexpr FResult Check(bool bInsideFormatSpec, int CurArgPos, const CharType* Fmt)
			{
				if (bInsideFormatSpec)
				{
					return {EFormatStringSanStatus::IncompleteFormatSpecifierOrUnescapedPercent, CurArgPos};
				}

				while (*Fmt)
				{
					if (Fmt[0] != (CharType)'%')
					{
						++Fmt;
						continue;
					}

					if (Fmt[1] == (CharType)'%')
					{
						Fmt += 2;
						continue;
					}

					if (Fmt[1] == (CharType)'\0')
					{
						return {EFormatStringSanStatus::IncompleteFormatSpecifierOrUnescapedPercent, CurArgPos};
					}

					return {EFormatStringSanStatus::NotEnoughArguments, CurArgPos};
				}

				return {EFormatStringSanStatus::Ok, 0};
			}
		};

		template <typename CharType, typename... ArgTypes>
		struct TCheckedFormatStringPrivate
		{
			const CharType* FormatString;

#if UE_VALIDATE_FORMAT_STRINGS && defined(__cpp_consteval)
			template <int32 N>
			consteval TCheckedFormatStringPrivate(const CharType (&Fmt)[N])
				: FormatString{Fmt}
			{
				using Checker = TCheckFormatString<CharType, std::decay_t<ArgTypes>...>;
					FResult Result = Checker::Check(false, 0, Fmt);

					switch (Result.Status)
					{
						case EFormatStringSanStatus::Ok: break;

						#define X(Name, Desc) \
						case EFormatStringSanStatus::Name: PRINTF_FORMAT_STRING_ERROR(Desc); break;
						#include "FormatStringSanErrors.inl"
						#undef X
					}
				}

			void PRINTF_FORMAT_STRING_ERROR(const char*)
			{
				// this non-consteval function exists to trigger a compiler error when called from a
				// consteval function
			}

#else

			template <int32 N>
			TCheckedFormatStringPrivate(const CharType(&Fmt)[N])
				: FormatString{Fmt}
			{
				static_assert((TIsValidVariadicFunctionArg<ArgTypes>::Value && ...), "Invalid argument(s) passed to Printf");
			}

#endif // defined(__cpp_consteval)
		};
	} // namespace FormatStringSan
} // namespace UE::Core::Private

namespace UE::Core
{
	template <typename FmtType, typename... ArgTypes>
	using TCheckedFormatString = ::UE::Core::Private::FormatStringSan::TCheckedFormatStringPrivate<FmtType, TIdentity_T<ArgTypes>...>;
}
