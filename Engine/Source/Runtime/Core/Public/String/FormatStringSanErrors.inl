// Copyright Epic Games, Inc. All Rights Reserved.

/*
  This file enables inlining the 'visitation' of possible FormatStringSan errors. Inlining is
  required because, without it, compiler output for malformed format strings would not include
  details of the specific error.
 */

X(SNeedsDereferencedNarrowString, "'%s' expects dereferenced `FAnsiString` or `FUtf8String`.")
X(SNeedsDereferencedWideString, "'%s' expects dereferenced `FString`.")
X(SNeedsPtrButGotChar, "'%s' expects character pointer but got character. (Is there an extra '*' somewhere?)")
X(SNeedsNarrowCharPtrArg, "'%s' expects `ANSICHAR*` or `UTF8CHAR*`.")
X(SNeedsWideCharPtrArg, "'%s' expects `TCHAR*`.")
X(SNeedsNarrowCharPtrArgButGotWide, "'%s' expects `ANSICHAR*` or `UTF8CHAR*` but got `TCHAR*`; use '%ls'.")
X(SNeedsWideCharPtrArgButGotNarrow, "'%s' expects `TCHAR*` but got `ANSICHAR*` or `UTF8CHAR*`; use '%hs'.")
X(CapitalSNeedsDereferencedNarrowString, "'%S' expects dereferenced `FAnsiString` or `FUtf8String`.")
X(CapitalSNeedsDereferencedWideString, "'%S' expects dereferenced `FString`.")
X(CapitalSNeedsPtrButGotChar, "'%S' expects character pointer but got character. (Is there an extra '*' somewhere?)")
X(CapitalSNeedsNarrowCharPtrArg, "'%S' expects `ANSICHAR*` or `UTF8CHAR*`.")
X(CapitalSNeedsWideCharPtrArg, "'%S' expects `TCHAR*`.")
X(CapitalSNeedsNarrowCharPtrArgButGotWide, "'%S' expects `ANSICHAR*` or `UTF8CHAR*` but got `TCHAR*`; use '%s'.")
X(CapitalSNeedsWideCharPtrArgButGotNarrow, "'%S' expects `TCHAR*` but got `ANSICHAR*` or `UTF8CHAR*`; use '%s'.")
X(HSNeedsDereferencedNarrowString, "'%hs' expects dereferenced `FAnsiString` or `FUtf8String`.")
X(HSNeedsPtrButGotChar, "'%hs' expects character pointer but got character. (Is there an extra '*' somewhere?)")
X(HSNeedsNarrowCharPtrArg, "'%hs' expects `ANSICHAR*` or `UTF8CHAR*`.")
X(HSNeedsNarrowCharPtrArgButGotWideOnNarrowString, "'%hs' expects `ANSICHAR*` or `UTF8CHAR*` but got `TCHAR*`; use '%ls'.")
X(HSNeedsNarrowCharPtrArgButGotWideOnWideString, "'%hs' expects `ANSICHAR*` or `UTF8CHAR*` but got `TCHAR*`; use '%s'.")
X(LSNeedsDereferencedWideString, "'%ls' expects dereferenced `FString`.")
X(LSNeedsPtrButGotChar, "'%ls' expects character pointer but got character. (Is there an extra '*' somewhere?)")
X(LSNeedsWideCharPtrArg, "'%ls' expects `TCHAR*`.")
X(LSNeedsWideCharPtrArgButGotNarrowOnNarrowString, "'%ls' expects `TCHAR*` but got `ANSICHAR*` or `UTF8CHAR*`; use '%s'.")
X(LSNeedsWideCharPtrArgButGotNarrowOnWideString, "'%ls' expects `TCHAR*` but got `ANSICHAR*` or `UTF8CHAR*`; use '%hs'.")
X(IncompleteFormatSpecifierOrUnescapedPercent, "incomplete '%' escape sequence. (to print a percent sign, write '%%'.)")
X(CNeedsCharArgOnNarrowString, "'%c' expects `ANSICHAR` or `UTF8CHAR`.")
X(CNeedsCharArgOnWideString, "'%c' expects `TCHAR`.")
X(PNeedsPointerArg, "'%p' expects a pointer.")
X(FNeedsFloatOrDoubleArg, "'%f' expects `float` or `double`.")
X(InvalidFormatSpec, "unsupported '%' format specifier. (to print a percent sign, write '%%'.)")
X(NotEnoughArguments, "not enough arguments provided to format string.")
X(NotEnoughSpecifiers, "missing format specifier (Is there a missing '%' somewhere?)")
X(DNeedsIntegerArg, "'%d' expects integral arg (eg. `char`, `int`, `long`, etc.)")
X(ZNeedsIntegerArg, "'%z' expects integral arg (eg. `char`, `int`, `long`, etc.)")
X(ZNeedsIntegerSpec, "'%z' must have a suffix (eg. '%zd', '%zu', etc.)")
X(LNeedsIntegerArg, "'%l[ ]' expects integral arg (eg. `char`, `int`, `long`, etc.)")
X(HNeedsIntegerArg, "'%h[ ]' expects integral arg (eg. `char`, `int`, `long`, etc.)")
X(HHNeedsIntegerSpec, "'%hh[ ]' expects integral suffix (eg. '%hhd', '%hhu', etc.)")
X(HHNeedsIntegerArg, "'%hh[ ]' expects integral arg (eg. `char`, `int`, `long`, etc.)")
X(LLNeedsIntegerSpec, "'%ll[ ]' expects integral suffix (eg. '%hhd', '%hhu', etc.)")
X(LLNeedsIntegerArg, "'%ll[ ]' expects integral arg (eg. `char`, `int`, `long`, etc.)")
X(I64BadSpec, "'%I' must appear as '%I64' with an integral suffix (eg. '%I64d', '%I64u', etc.)")
X(I64NeedsIntegerArg, "'%I64[ ]' expects integral arg (eg. `char`, `int`, `long`, etc.)")
X(DynamicLengthSpecNeedsIntegerArg, "dynamic field width specifier '*' expects integral arg (eg. `char`, `int`, `long`, etc.)")
