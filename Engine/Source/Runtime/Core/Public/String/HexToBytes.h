// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringFwd.h"

namespace UE::String
{

/** 
 * Convert an array of hex digits into an array of bytes.
 *
 * @param Hex Array of hex digits to convert.
 * @param OutBytes [out] Array of at least (Hex.Len()+1)/2 output bytes.
 *
 * @return Number of bytes written to the output.
 */
CORE_API int32 HexToBytes(FWideStringView Hex, uint8* OutBytes);
CORE_API int32 HexToBytes(FUtf8StringView Hex, uint8* OutBytes);

} // UE::String
