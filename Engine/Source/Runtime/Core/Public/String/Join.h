// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringView.h"
#include "Misc/StringBuilder.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/Tuple.h"

namespace UE::String::Private
{

template <typename RangeType, typename ProjectionType, typename DelimiterType, typename QuoteType>
struct TJoinQuotedBy
{
	RangeType&& Range;
	ProjectionType Projection;
	DelimiterType&& Delimiter;
	QuoteType&& Quote;
};

template <typename RangeType, typename ProjectionType, typename DelimiterType>
struct TJoinBy
{
	RangeType&& Range;
	ProjectionType Projection;
	DelimiterType&& Delimiter;
};

template <typename TupleType, typename ProjectionType, typename DelimiterType, typename QuoteType>
struct TJoinTupleQuotedBy
{
	TupleType&& Tuple;
	ProjectionType Projection;
	DelimiterType&& Delimiter;
	QuoteType&& Quote;
};

template <typename TupleType, typename ProjectionType, typename DelimiterType>
struct TJoinTupleBy
{
	TupleType&& Tuple;
	ProjectionType Projection;
	DelimiterType&& Delimiter;
};

} // UE::String::Private

namespace UE::String
{

/**
 * Create an object that can be appended to a string builder to append every element of the range
 * to the builder, separating the elements by the delimiter and surrounding every element on both
 * sides with a quote.
 *
 * @param Range        The range of elements to join and append.
 * @param Projection   The projection to apply to the elements before appending them.
 * @param Delimiter    The delimiter to append as a separator for the elements.
 * @param Quote        The quote to append on both sides of each element.
 *
 * @return An anonymous object to append to a string builder.
 */
template <typename RangeType, typename ProjectionType, typename DelimiterType, typename QuoteType>
inline auto JoinQuotedBy(RangeType&& Range, ProjectionType Projection, DelimiterType&& Delimiter, QuoteType&& Quote)
	-> Private::TJoinQuotedBy<RangeType, ProjectionType, DelimiterType, QuoteType>
{
	return {Forward<RangeType>(Range), MoveTemp(Projection), Forward<DelimiterType>(Delimiter), Forward<QuoteType>(Quote)};
}

/**
 * Append every element of the range to the builder, separating the elements by the delimiter, and
 * surrounding every element on each side with the given quote.
 *
 * @param Range        The range of elements to join and append.
 * @param Projection   The projection to apply to the elements before appending them.
 * @param Delimiter    The delimiter to append as a separator for the elements.
 * @param Quote        The quote to append on both sides of each element.
 * @param Builder      The builder to append to.
 *
 * @return The builder, to allow additional operations to be composed with this one.
 */
template <typename RangeType, typename ProjectionType, typename DelimiterType, typename QuoteType, typename CharType>
inline TStringBuilderBase<CharType>& JoinQuotedByTo(
	RangeType&& Range,
	ProjectionType Projection,
	DelimiterType&& Delimiter,
	QuoteType&& Quote,
	TStringBuilderBase<CharType>& Builder)
{
	bool bFirst = true;
	for (auto&& Elem : Forward<RangeType>(Range))
	{
		if (bFirst)
		{
			bFirst = false;
		}
		else
		{
			Builder << Delimiter;
		}
		Builder << Quote << Invoke(Projection, Elem) << Quote;
	}
	return Builder;
}

/**
 * Create an object that can be appended to a string builder to append every element of the range
 * to the builder, separating the elements by the delimiter and surrounding every element on both
 * sides with a quote.
 *
 * @param Range       The range of elements to join and append.
 * @param Delimiter   The delimiter to append as a separator for the elements.
 * @param Quote       The quote to append on both sides of each element.
 *
 * @return An anonymous object to append to a string builder.
 */
template <typename RangeType, typename DelimiterType, typename QuoteType>
inline auto JoinQuoted(RangeType&& Range, DelimiterType&& Delimiter, QuoteType&& Quote)
	-> Private::TJoinQuotedBy<RangeType, FIdentityFunctor, DelimiterType, QuoteType>
{
	return {Forward<RangeType>(Range), FIdentityFunctor(), Forward<DelimiterType>(Delimiter), Forward<QuoteType>(Quote)};
}

/**
 * Append every element of the range to the builder, separating the elements by the delimiter, and
 * surrounding every element on each side with the given quote.
 *
 * @param Range       The range of elements to join and append.
 * @param Delimiter   The delimiter to append as a separator for the elements.
 * @param Quote       The quote to append on both sides of each element.
 * @param Builder     The builder to append to.
 *
 * @return The builder, to allow additional operations to be composed with this one.
 */
template <typename RangeType, typename DelimiterType, typename QuoteType, typename CharType>
inline TStringBuilderBase<CharType>& JoinQuotedTo(
	RangeType&& Range,
	DelimiterType&& Delimiter,
	QuoteType&& Quote,
	TStringBuilderBase<CharType>& Builder)
{
	return JoinQuotedByTo(Forward<RangeType>(Range), FIdentityFunctor(), Forward<DelimiterType>(Delimiter), Forward<QuoteType>(Quote), Builder);
}

/**
 * Create an object that can be appended to a string builder to append every element of the range
 * to the builder, separating the elements by the delimiter.
 *
 * @param Range        The range of elements to join and append.
 * @param Projection   The projection to apply to the elements before appending them.
 * @param Delimiter    The delimiter to append as a separator for the elements.
 *
 * @return An anonymous object to append to a string builder.
 */
template <typename RangeType, typename ProjectionType, typename DelimiterType>
inline auto JoinBy(RangeType&& Range, ProjectionType Projection, DelimiterType&& Delimiter)
	-> Private::TJoinBy<RangeType, ProjectionType, DelimiterType>
{
	return {Forward<RangeType>(Range), MoveTemp(Projection), Forward<DelimiterType>(Delimiter)};
}

/**
 * Append every element of the range to the builder, separating the elements by the delimiter.
 *
 * @param Range        The range of elements to join and append.
 * @param Projection   The projection to apply to the elements before appending them.
 * @param Delimiter    The delimiter to append as a separator for the elements.
 * @param Builder      The builder to append to.
 *
 * @return The builder, to allow additional operations to be composed with this one.
 */
template <typename RangeType, typename ProjectionType, typename DelimiterType, typename CharType>
inline TStringBuilderBase<CharType>& JoinByTo(
	RangeType&& Range,
	ProjectionType Projection,
	DelimiterType&& Delimiter,
	TStringBuilderBase<CharType>& Builder)
{
	bool bFirst = true;
	for (auto&& Elem : Forward<RangeType>(Range))
	{
		if (bFirst)
		{
			bFirst = false;
		}
		else
		{
			Builder << Delimiter;
		}
		Builder << Invoke(Projection, Elem);
	}
	return Builder;
}

/**
 * Create an object that can be appended to a string builder to append every element of the range
 * to the builder, separating the elements by the delimiter.
 *
 * @param Range        The range of elements to join and append.
 * @param Delimiter    The delimiter to append as a separator for the elements.
 *
 * @return An anonymous object to append to a string builder.
 */
template <typename RangeType, typename DelimiterType>
inline auto Join(RangeType&& Range, DelimiterType&& Delimiter)
	-> Private::TJoinBy<RangeType, FIdentityFunctor, DelimiterType>
{
	return {Forward<RangeType>(Range), FIdentityFunctor(), Forward<DelimiterType>(Delimiter)};
}

/**
 * Append every element of the range to the builder, separating the elements by the delimiter.
 *
 * @param Range        The range of elements to join and append.
 * @param Delimiter    The delimiter to append as a separator for the elements.
 * @param Builder      The builder to append to.
 *
 * @return The builder, to allow additional operations to be composed with this one.
 */
template <typename RangeType, typename DelimiterType, typename CharType>
inline TStringBuilderBase<CharType>& JoinTo(RangeType&& Range, DelimiterType&& Delimiter, TStringBuilderBase<CharType>& Builder)
{
	return JoinByTo(Forward<RangeType>(Range), FIdentityFunctor(), Forward<DelimiterType>(Delimiter), Builder);
}

/**
 * Create an object that can be appended to a string builder to append every element of the tuple
 * to the builder, separating the elements by the delimiter and surrounding every element on both
 * sides with a quote.
 *
 * @param Tuple        The tuple of elements to join and append.
 * @param Projection   The projection to apply to the elements before appending them.
 * @param Delimiter    The delimiter to append as a separator for the elements.
 * @param Quote        The quote to append on both sides of each element.
 *
 * @return An anonymous object to append to a string builder.
 */
template <typename TupleType, typename ProjectionType, typename DelimiterType, typename QuoteType>
inline auto JoinTupleQuotedBy(TupleType&& Tuple, ProjectionType Projection, DelimiterType&& Delimiter, QuoteType&& Quote)
	-> Private::TJoinTupleQuotedBy<TupleType, ProjectionType, DelimiterType, QuoteType>
{
	return {Forward<TupleType>(Tuple), MoveTemp(Projection), Forward<DelimiterType>(Delimiter), Forward<QuoteType>(Quote)};
}

/**
 * Append every element of the tuple to the builder, separating the elements by the delimiter, and
 * surrounding every element on each side with the given quote.
 *
 * @param Tuple        The tuple of elements to join and append.
 * @param Projection   The projection to apply to the elements before appending them.
 * @param Delimiter    The delimiter to append as a separator for the elements.
 * @param Quote        The quote to append on both sides of each element.
 * @param Builder      The builder to append to.
 *
 * @return The builder, to allow additional operations to be composed with this one.
 */
template <typename TupleType, typename ProjectionType, typename DelimiterType, typename QuoteType, typename CharType
	UE_REQUIRES(TIsTuple_V<std::decay_t<TupleType>>)>
inline TStringBuilderBase<CharType>& JoinTupleQuotedByTo(
	TupleType&& Tuple,
	ProjectionType Projection,
	DelimiterType&& Delimiter,
	QuoteType&& Quote,
	TStringBuilderBase<CharType>& Builder)
{
	bool bFirst = true;
	VisitTupleElements([&bFirst, &Projection, &Delimiter, &Quote, &Builder](auto&& Elem)
	{
		if (bFirst)
		{
			bFirst = false;
		}
		else
		{
			Builder << Delimiter;
		}
		Builder << Quote << Invoke(Projection, Elem) << Quote;
	}, Tuple);
	return Builder;
}

/**
 * Create an object that can be appended to a string builder to append every element of the tuple
 * to the builder, separating the elements by the delimiter and surrounding every element on both
 * sides with a quote.
 *
 * @param Tuple       The tuple of elements to join and append.
 * @param Delimiter   The delimiter to append as a separator for the elements.
 * @param Quote       The quote to append on both sides of each element.
 *
 * @return An anonymous object to append to a string builder.
 */
template <typename TupleType, typename DelimiterType, typename QuoteType>
inline auto JoinTupleQuoted(TupleType&& Tuple, DelimiterType&& Delimiter, QuoteType&& Quote)
	-> Private::TJoinTupleQuotedBy<TupleType, FIdentityFunctor, DelimiterType, QuoteType>
{
	return {Forward<TupleType>(Tuple), FIdentityFunctor(), Forward<DelimiterType>(Delimiter), Forward<QuoteType>(Quote)};
}

/**
 * Append every element of the tuple to the builder, separating the elements by the delimiter, and
 * surrounding every element on each side with the given quote.
 *
 * @param Tuple       The tuple of elements to join and append.
 * @param Delimiter   The delimiter to append as a separator for the elements.
 * @param Quote       The quote to append on both sides of each element.
 * @param Builder     The builder to append to.
 *
 * @return The builder, to allow additional operations to be composed with this one.
 */
template <typename TupleType, typename DelimiterType, typename QuoteType, typename CharType>
inline TStringBuilderBase<CharType>& JoinTupleQuotedTo(
	TupleType&& Tuple,
	DelimiterType&& Delimiter,
	QuoteType&& Quote,
	TStringBuilderBase<CharType>& Builder)
{
	return JoinTupleQuotedByTo(Forward<TupleType>(Tuple), FIdentityFunctor(), Forward<DelimiterType>(Delimiter), Forward<QuoteType>(Quote), Builder);
}

/**
 * Create an object that can be appended to a string builder to append every element of the tuple
 * to the builder, separating the elements by the delimiter.
 *
 * @param Tuple        The tuple of elements to join and append.
 * @param Projection   The projection to apply to the elements before appending them.
 * @param Delimiter    The delimiter to append as a separator for the elements.
 *
 * @return An anonymous object to append to a string builder.
 */
template <typename TupleType, typename ProjectionType, typename DelimiterType>
inline auto JoinTupleBy(TupleType&& Tuple, ProjectionType Projection, DelimiterType&& Delimiter)
	-> Private::TJoinTupleBy<TupleType, ProjectionType, DelimiterType>
{
	return {Forward<TupleType>(Tuple), MoveTemp(Projection), Forward<DelimiterType>(Delimiter)};
}

/**
 * Append every element of the tuple to the builder, separating the elements by the delimiter.
 *
 * @param Tuple        The tuple of elements to join and append.
 * @param Projection   The projection to apply to the elements before appending them.
 * @param Delimiter    The delimiter to append as a separator for the elements.
 * @param Builder      The builder to append to.
 *
 * @return The builder, to allow additional operations to be composed with this one.
 */
template <typename TupleType, typename ProjectionType, typename DelimiterType, typename CharType
	UE_REQUIRES(TIsTuple_V<std::decay_t<TupleType>>)>
inline TStringBuilderBase<CharType>& JoinTupleByTo(
	TupleType&& Tuple,
	ProjectionType Projection,
	DelimiterType&& Delimiter,
	TStringBuilderBase<CharType>& Builder)
{
	bool bFirst = true;
	VisitTupleElements([&bFirst, &Projection, &Delimiter, &Builder](auto&& Elem)
	{
		if (bFirst)
		{
			bFirst = false;
		}
		else
		{
			Builder << Delimiter;
		}
		Builder << Invoke(Projection, Elem);
	}, Tuple);
	return Builder;
}

/**
 * Create an object that can be appended to a string builder to append every element of the tuple
 * to the builder, separating the elements by the delimiter.
 *
 * @param Tuple        The tuple of elements to join and append.
 * @param Delimiter    The delimiter to append as a separator for the elements.
 *
 * @return An anonymous object to append to a string builder.
 */
template <typename TupleType, typename DelimiterType>
inline auto JoinTuple(TupleType&& Tuple, DelimiterType&& Delimiter)
	-> Private::TJoinTupleBy<TupleType, FIdentityFunctor, DelimiterType>
{
	return {Forward<TupleType>(Tuple), FIdentityFunctor(), Forward<DelimiterType>(Delimiter)};
}

/**
 * Append every element of the tuple to the builder, separating the elements by the delimiter.
 *
 * @param Tuple        The tuple of elements to join and append.
 * @param Delimiter    The delimiter to append as a separator for the elements.
 * @param Builder      The builder to append to.
 *
 * @return The builder, to allow additional operations to be composed with this one.
 */
template <typename TupleType, typename DelimiterType, typename CharType>
inline TStringBuilderBase<CharType>& JoinTupleTo(TupleType&& Tuple, DelimiterType&& Delimiter, TStringBuilderBase<CharType>& Builder)
{
	return JoinTupleByTo(Forward<TupleType>(Tuple), FIdentityFunctor(), Forward<DelimiterType>(Delimiter), Builder);
}

} // UE::String

namespace UE::String::Private
{

template <typename RangeType, typename ProjectionType, typename DelimiterType, typename QuoteType, typename CharType>
inline TStringBuilderBase<CharType>& operator<<(
	TStringBuilderBase<CharType>& Builder,
	Private::TJoinQuotedBy<RangeType, ProjectionType, DelimiterType, QuoteType>&& Adapter)
{
	return JoinQuotedByTo(
		Forward<RangeType>(Adapter.Range),
		MoveTemp(Adapter.Projection),
		Forward<DelimiterType>(Adapter.Delimiter),
		Forward<QuoteType>(Adapter.Quote),
		Builder);
}

template <typename RangeType, typename ProjectionType, typename DelimiterType, typename CharType>
inline TStringBuilderBase<CharType>& operator<<(
	TStringBuilderBase<CharType>& Builder,
	Private::TJoinBy<RangeType, ProjectionType, DelimiterType>&& Adapter)
{
	return JoinByTo(Forward<RangeType>(Adapter.Range), MoveTemp(Adapter.Projection), Forward<DelimiterType>(Adapter.Delimiter), Builder);
}

template <typename TupleType, typename ProjectionType, typename DelimiterType, typename QuoteType, typename CharType>
inline TStringBuilderBase<CharType>& operator<<(
	TStringBuilderBase<CharType>& Builder,
	Private::TJoinTupleQuotedBy<TupleType, ProjectionType, DelimiterType, QuoteType>&& Adapter)
{
	return JoinTupleQuotedByTo(
		Forward<TupleType>(Adapter.Tuple),
		MoveTemp(Adapter.Projection),
		Forward<DelimiterType>(Adapter.Delimiter),
		Forward<QuoteType>(Adapter.Quote),
		Builder);
}

template <typename TupleType, typename ProjectionType, typename DelimiterType, typename CharType>
inline TStringBuilderBase<CharType>& operator<<(
	TStringBuilderBase<CharType>& Builder,
	Private::TJoinTupleBy<TupleType, ProjectionType, DelimiterType>&& Adapter)
{
	return JoinTupleByTo(Forward<TupleType>(Adapter.Tuple), MoveTemp(Adapter.Projection), Forward<DelimiterType>(Adapter.Delimiter), Builder);
}

} // UE::String::Private
