// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringFwd.h"

CORE_API void LexFromString(int8& OutValue,   const FStringView& InString);
CORE_API void LexFromString(int16& OutValue,  const FStringView& InString);
CORE_API void LexFromString(int32& OutValue,  const FStringView& InString);
CORE_API void LexFromString(int64& OutValue,  const FStringView& InString);
CORE_API void LexFromString(uint8& OutValue,  const FStringView& InString);
CORE_API void LexFromString(uint16& OutValue, const FStringView& InString);
CORE_API void LexFromString(uint32& OutValue, const FStringView& InString);
CORE_API void LexFromString(uint64& OutValue, const FStringView& InString);
CORE_API void LexFromString(float& OutValue,  const FStringView& InString);
CORE_API void LexFromString(double& OutValue, const FStringView& InString);
CORE_API void LexFromString(bool& OutValue,   const FStringView& InString);

CORE_API void LexFromString(int8& OutValue,   const FUtf8StringView& InString);
CORE_API void LexFromString(int16& OutValue,  const FUtf8StringView& InString);
CORE_API void LexFromString(int32& OutValue,  const FUtf8StringView& InString);
CORE_API void LexFromString(int64& OutValue,  const FUtf8StringView& InString);
CORE_API void LexFromString(uint8& OutValue,  const FUtf8StringView& InString);
CORE_API void LexFromString(uint16& OutValue, const FUtf8StringView& InString);
CORE_API void LexFromString(uint32& OutValue, const FUtf8StringView& InString);
CORE_API void LexFromString(uint64& OutValue, const FUtf8StringView& InString);
CORE_API void LexFromString(float& OutValue,  const FUtf8StringView& InString);
CORE_API void LexFromString(double& OutValue, const FUtf8StringView& InString);
CORE_API void LexFromString(bool& OutValue,   const FUtf8StringView& InString);
