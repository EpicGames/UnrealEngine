// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"

namespace UE::String
{
	/** Replaces all Line Endings with "\n" line terminator */
	CORE_API FString FromHostLineEndings(const FString& InString);
	CORE_API FString FromHostLineEndings(FString&& InString);
	CORE_API void FromHostLineEndingsInline(FString& InString);

	/** Replaces all Line Endings with the host platform line terminator */
	CORE_API FString ToHostLineEndings(const FString& InString);
	CORE_API FString ToHostLineEndings(FString&& InString);
	CORE_API void ToHostLineEndingsInline(FString& InString);

} // UE::String
