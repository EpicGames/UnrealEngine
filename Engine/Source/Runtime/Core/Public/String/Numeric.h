// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"

namespace UE::String
{

/**
 * Checks if the passed in string contains only digits, at most one dot, and optionally a +/- sign at the start.
 *
 * @param View The string to check.
 *
 * @return True if the string looks like a valid floating point number.
 */
CORE_API bool IsNumeric(FWideStringView View);
CORE_API bool IsNumeric(FUtf8StringView View);

/**
 * Checks if the passed in string contains only digits.
 *
 * @param View The string to check.
 *
 * @return True if the string only contains numeric characters.
 */
CORE_API bool IsNumericOnlyDigits(FWideStringView View);
CORE_API bool IsNumericOnlyDigits(FUtf8StringView View);

}
