// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"

template <typename FuncType> class TFunctionRef;

namespace UE::String
{

enum class EParseLinesOptions : uint32
{
	/** Use the default options when parsing lines. */
	None = 0,
	/** Skip lines that are empty or, if trimming, only whitespace. */
	SkipEmpty = 1 << 0,
	/** Trim whitespace from each parsed line. */
	Trim = 1 << 1,
};

ENUM_CLASS_FLAGS(EParseLinesOptions);

/**
 * Visit every line in the input string as terminated by any of CRLF, CR, LF.
 *
 * By default, empty lines are visited.
 *
 * @param View      A view of the string to split into lines.
 * @param Visitor   A function that is called for each line.
 * @param Options   Flags to modify the default behavior.
 */
CORE_API void ParseLines(
	FWideStringView View,
	TFunctionRef<void (FWideStringView)> Visitor,
	EParseLinesOptions Options = EParseLinesOptions::None);
CORE_API void ParseLines(
	FUtf8StringView View,
	TFunctionRef<void (FUtf8StringView)> Visitor,
	EParseLinesOptions Options = EParseLinesOptions::None);

/**
 * Parse lines in the input string as terminated by any of CRLF, CR, LF.
 *
 * Output strings are sub-views of the input view and have the same lifetime as the input view.
 * By default, empty lines are collected.
 *
 * @param View      A view of the string to split into lines.
 * @param Output    The output to add parsed lines to by calling Output.Add(FStringView).
 * @param Options   Flags to modify the default behavior.
 */
template <typename OutputType, typename StringViewType>
inline void ParseLines(
	const StringViewType View,
	OutputType& Output,
	const EParseLinesOptions Options = EParseLinesOptions::None)
{
	ParseLines(View, [&Output](StringViewType Line) { Output.Add(Line); }, Options);
}

} // UE::String
