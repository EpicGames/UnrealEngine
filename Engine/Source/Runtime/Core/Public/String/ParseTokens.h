// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"

template <typename FuncType> class TFunctionRef;

namespace UE::String
{

enum class EParseTokensOptions : uint32
{
	/** Use the default options when parsing tokens. */
	None = 0,
	/** Ignore case when comparing delimiters against the string. */
	IgnoreCase = 1 << 0,
	/** Skip tokens that are empty or, if trimming, only whitespace. */
	SkipEmpty = 1 << 1,
	/** Trim whitespace from each parsed token. */
	Trim = 1 << 2,
};

ENUM_CLASS_FLAGS(EParseTokensOptions);

/**
 * Visit every token in the input string, as separated by the delimiter.
 *
 * By default, comparisons with the delimiter are case-sensitive and empty tokens are visited.
 *
 * @param View        A view of the string to split into tokens.
 * @param Delimiter   A delimiter character to split on.
 * @param Visitor     A function that is called for each token.
 * @param Options     Flags to modify the default behavior.
 */
CORE_API void ParseTokens(
	FAnsiStringView View,
	ANSICHAR Delimiter,
	TFunctionRef<void (FAnsiStringView)> Visitor,
	EParseTokensOptions Options = EParseTokensOptions::None);
CORE_API void ParseTokens(
	FWideStringView View,
	WIDECHAR Delimiter,
	TFunctionRef<void (FWideStringView)> Visitor,
	EParseTokensOptions Options = EParseTokensOptions::None);
CORE_API void ParseTokens(
	FUtf8StringView View,
	UTF8CHAR Delimiter,
	TFunctionRef<void (FUtf8StringView)> Visitor,
	EParseTokensOptions Options = EParseTokensOptions::None);

/**
 * Parses every token in the input string, as separated by the delimiter.
 *
 * Output strings are sub-views of the input view and have the same lifetime as the input view.
 * By default, comparisons with the delimiter are case-sensitive and empty tokens are visited.
 *
 * @param View        A view of the string to split into tokens.
 * @param Delimiter   A delimiter character to split on.
 * @param Output      The output to add parsed tokens to by calling Output.Add(FStringView).
 * @param Options     Flags to modify the default behavior.
 */
template <typename OutputType>
inline void ParseTokens(
	const FAnsiStringView View,
	const ANSICHAR Delimiter,
	OutputType& Output,
	const EParseTokensOptions Options = EParseTokensOptions::None)
{
	return ParseTokens(View, Delimiter, [&Output](FAnsiStringView Token) { Output.Add(Token); }, Options);
}
template <typename OutputType>
inline void ParseTokens(
	const FWideStringView View,
	const WIDECHAR Delimiter,
	OutputType& Output,
	const EParseTokensOptions Options = EParseTokensOptions::None)
{
	return ParseTokens(View, Delimiter, [&Output](FWideStringView Token) { Output.Add(Token); }, Options);
}
template <typename OutputType>
inline void ParseTokens(
	const FUtf8StringView View,
	const UTF8CHAR Delimiter,
	OutputType& Output,
	const EParseTokensOptions Options = EParseTokensOptions::None)
{
	return ParseTokens(View, Delimiter, [&Output](FUtf8StringView Token) { Output.Add(Token); }, Options);
}

/**
 * Visit every token in the input string, as separated by the delimiter.
 *
 * By default, comparisons with the delimiter are case-sensitive and empty tokens are visited.
 *
 * @param View        A view of the string to split into tokens.
 * @param Delimiter   A non-empty delimiter to split on.
 * @param Visitor     A function that is called for each token.
 * @param Options     Flags to modify the default behavior.
 */
CORE_API void ParseTokens(
	FAnsiStringView View,
	FAnsiStringView Delimiter,
	TFunctionRef<void (FAnsiStringView)> Visitor,
	EParseTokensOptions Options = EParseTokensOptions::None);
CORE_API void ParseTokens(
	FWideStringView View,
	FWideStringView Delimiter,
	TFunctionRef<void (FWideStringView)> Visitor,
	EParseTokensOptions Options = EParseTokensOptions::None);
CORE_API void ParseTokens(
	FUtf8StringView View,
	FUtf8StringView Delimiter,
	TFunctionRef<void (FUtf8StringView)> Visitor,
	EParseTokensOptions Options = EParseTokensOptions::None);

/**
 * Parses every token in the input string, as separated by the delimiter.
 *
 * Output strings are sub-views of the input view and have the same lifetime as the input view.
 * By default, comparisons with the delimiter are case-sensitive and empty tokens are visited.
 *
 * @param View        A view of the string to split into tokens.
 * @param Delimiter   A non-empty delimiter to split on.
 * @param Output      The output to add parsed tokens to by calling Output.Add(FStringView).
 * @param Options     Flags to modify the default behavior.
 */
template <typename OutputType>
inline void ParseTokens(
	const FAnsiStringView View,
	const FAnsiStringView Delimiter,
	OutputType& Output,
	const EParseTokensOptions Options = EParseTokensOptions::None)
{
	return ParseTokens(View, Delimiter, [&Output](FAnsiStringView Token) { Output.Add(Token); }, Options);
}
template <typename OutputType>
inline void ParseTokens(
	const FWideStringView View,
	const FWideStringView Delimiter,
	OutputType& Output,
	const EParseTokensOptions Options = EParseTokensOptions::None)
{
	return ParseTokens(View, Delimiter, [&Output](FWideStringView Token) { Output.Add(Token); }, Options);
}
template <typename OutputType>
inline void ParseTokens(
	const FUtf8StringView View,
	const FUtf8StringView Delimiter,
	OutputType& Output,
	const EParseTokensOptions Options = EParseTokensOptions::None)
{
	return ParseTokens(View, Delimiter, [&Output](FUtf8StringView Token) { Output.Add(Token); }, Options);
}

/**
 * Visit every token in the input string, as separated by any of the delimiters.
 *
 * By default, comparisons with delimiters are case-sensitive and empty tokens are visited.
 *
 * @param View         A view of the string to split into tokens.
 * @param Delimiters   An array of delimiter characters to split on.
 * @param Visitor      A function that is called for each token.
 * @param Options      Flags to modify the default behavior.
 */
CORE_API void ParseTokensMultiple(
	FAnsiStringView View,
	TConstArrayView<ANSICHAR> Delimiters,
	TFunctionRef<void (FAnsiStringView)> Visitor,
	EParseTokensOptions Options = EParseTokensOptions::None);
CORE_API void ParseTokensMultiple(
	FWideStringView View,
	TConstArrayView<WIDECHAR> Delimiters,
	TFunctionRef<void (FWideStringView)> Visitor,
	EParseTokensOptions Options = EParseTokensOptions::None);
CORE_API void ParseTokensMultiple(
	FUtf8StringView View,
	TConstArrayView<UTF8CHAR> Delimiters,
	TFunctionRef<void (FUtf8StringView)> Visitor,
	EParseTokensOptions Options = EParseTokensOptions::None);

/**
 * Parses every token in the input string, as separated by any of the delimiters.
 *
 * Output strings are sub-views of the input view and have the same lifetime as the input view.
 * By default, comparisons with delimiters are case-sensitive and empty tokens are visited.
 *
 * @param View         A view of the string to split into tokens.
 * @param Delimiters   An array of delimiter characters to split on.
 * @param Output       The output to add parsed tokens to by calling Output.Add(FStringView).
 * @param Options      Flags to modify the default behavior.
 */
template <typename OutputType>
inline void ParseTokensMultiple(
	const FAnsiStringView View,
	const TConstArrayView<ANSICHAR> Delimiters,
	OutputType& Output,
	const EParseTokensOptions Options = EParseTokensOptions::None)
{
	ParseTokensMultiple(View, Delimiters, [&Output](FAnsiStringView Token) { Output.Add(Token); }, Options);
}
template <typename OutputType>
inline void ParseTokensMultiple(
	const FWideStringView View,
	const TConstArrayView<WIDECHAR> Delimiters,
	OutputType& Output,
	const EParseTokensOptions Options = EParseTokensOptions::None)
{
	ParseTokensMultiple(View, Delimiters, [&Output](FWideStringView Token) { Output.Add(Token); }, Options);
}
template <typename OutputType>
inline void ParseTokensMultiple(
	const FUtf8StringView View,
	const TConstArrayView<UTF8CHAR> Delimiters,
	OutputType& Output,
	const EParseTokensOptions Options = EParseTokensOptions::None)
{
	ParseTokensMultiple(View, Delimiters, [&Output](FUtf8StringView Token) { Output.Add(Token); }, Options);
}

/**
 * Visit every token in the input string, as separated by any of the delimiters.
 *
 * By default, comparisons with delimiters are case-sensitive and empty tokens are visited.
 * Behavior is undefined when delimiters overlap each other, such as the delimiters
 * ("AB, "BC") and the input string "1ABC2".
 *
 * @param View         A view of the string to split into tokens.
 * @param Delimiters   An array of non-overlapping non-empty delimiters to split on.
 * @param Visitor      A function that is called for each token.
 * @param Options      Flags to modify the default behavior.
 */
CORE_API void ParseTokensMultiple(
	FAnsiStringView View,
	TConstArrayView<FAnsiStringView> Delimiters,
	TFunctionRef<void (FAnsiStringView)> Visitor,
	EParseTokensOptions Options = EParseTokensOptions::None);
CORE_API void ParseTokensMultiple(
	FWideStringView View,
	TConstArrayView<FWideStringView> Delimiters,
	TFunctionRef<void (FWideStringView)> Visitor,
	EParseTokensOptions Options = EParseTokensOptions::None);
CORE_API void ParseTokensMultiple(
	FUtf8StringView View,
	TConstArrayView<FUtf8StringView> Delimiters,
	TFunctionRef<void (FUtf8StringView)> Visitor,
	EParseTokensOptions Options = EParseTokensOptions::None);

/**
 * Parses every token in the input string, as separated by any of the delimiters.
 *
 * Output strings are sub-views of the input view and have the same lifetime as the input view.
 * By default, comparisons with delimiters are case-sensitive and empty tokens are visited.
 * Behavior is undefined when delimiters overlap each other, such as the delimiters
 * ("AB, "BC") and the input string "1ABC2".
 *
 * @param View         A view of the string to split into tokens.
 * @param Delimiters   An array of non-overlapping non-empty delimiters to split on.
 * @param Output       The output to add parsed tokens to by calling Output.Add(FStringView).
 * @param Options      Flags to modify the default behavior.
 */
template <typename OutputType>
inline void ParseTokensMultiple(
	const FAnsiStringView View,
	const TConstArrayView<FAnsiStringView> Delimiters,
	OutputType& Output,
	const EParseTokensOptions Options = EParseTokensOptions::None)
{
	ParseTokensMultiple(View, Delimiters, [&Output](FAnsiStringView Token) { Output.Add(Token); }, Options);
}
template <typename OutputType>
inline void ParseTokensMultiple(
	const FWideStringView View,
	const TConstArrayView<FWideStringView> Delimiters,
	OutputType& Output,
	const EParseTokensOptions Options = EParseTokensOptions::None)
{
	ParseTokensMultiple(View, Delimiters, [&Output](FWideStringView Token) { Output.Add(Token); }, Options);
}
template <typename OutputType>
inline void ParseTokensMultiple(
	const FUtf8StringView View,
	const TConstArrayView<FUtf8StringView> Delimiters,
	OutputType& Output,
	const EParseTokensOptions Options = EParseTokensOptions::None)
{
	ParseTokensMultiple(View, Delimiters, [&Output](FUtf8StringView Token) { Output.Add(Token); }, Options);
}

} // UE::String
