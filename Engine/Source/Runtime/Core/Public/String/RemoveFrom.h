// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringFwd.h"
#include "Misc/CString.h"

namespace UE::String
{
	template <typename CharType>
	[[nodiscard]] inline TStringView<CharType> RemoveFromStart(const TStringView<CharType> View, const TStringView<CharType> Prefix, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive)
	{
		return View.StartsWith(Prefix, SearchCase) ? View.RightChop(Prefix.Len()) : View;
	}

	template <typename CharType>
	[[nodiscard]] inline TStringView<CharType> RemoveFromEnd(const TStringView<CharType> View, const TStringView<CharType> Prefix, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive)
	{
		return View.EndsWith(Prefix, SearchCase) ? View.LeftChop(Prefix.Len()) : View;
	}
} // UE::String
