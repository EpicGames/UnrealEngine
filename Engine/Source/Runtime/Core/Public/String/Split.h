// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "CoreTypes.h"
#include "Misc/CString.h"

namespace UE::String
{

/**
 * Split the view on the first occurrence of the search string.
 *
 * @param View The string to split.
 * @param Search The string to search for and split on.
 * @param OutLeft Receives the part of the view to the left of the search string if it is found.
 * @param OutRight Receives the part of the view to the right of the search string if it is found.
 * @param SearchCase Whether the comparison should ignore case.
 * @return true if the search string was found and the outputs were written, otherwise false.
 */
CORE_API bool SplitFirst(FAnsiStringView View, FAnsiStringView Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);
CORE_API bool SplitFirst(FUtf8StringView View, FUtf8StringView Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);
CORE_API bool SplitFirst(FWideStringView View, FWideStringView Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);

/**
 * Split the view on the last occurrence of the search string.
 *
 * @param View The string to split.
 * @param Search The string to search for and split on.
 * @param OutLeft Receives the part of the view to the left of the search string if it is found.
 * @param OutRight Receives the part of the view to the right of the search string if it is found.
 * @param SearchCase Whether the comparison should ignore case.
 * @return true if the search string was found and the outputs were written, otherwise false.
 */
CORE_API bool SplitLast(FAnsiStringView View, FAnsiStringView Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);
CORE_API bool SplitLast(FUtf8StringView View, FUtf8StringView Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);
CORE_API bool SplitLast(FWideStringView View, FWideStringView Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);

/**
 * Split the view on the first occurrence of any search string.
 *
 * @param View The string to split.
 * @param Search The strings to search for and split on.
 * @param OutLeft Receives the part of the view to the left of the search string if it is found.
 * @param OutRight Receives the part of the view to the right of the search string if it is found.
 * @param SearchCase Whether the comparison should ignore case.
 * @param OutMatchIndex Receives the index of the search string that matched.
 * @return true if the search string was found and the outputs were written, otherwise false.
 */
CORE_API bool SplitFirstOfAny(FAnsiStringView View, TConstArrayView<FAnsiStringView> Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API bool SplitFirstOfAny(FUtf8StringView View, TConstArrayView<FUtf8StringView> Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API bool SplitFirstOfAny(FWideStringView View, TConstArrayView<FWideStringView> Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);

/**
 * Split the view on the last occurrence of any search string.
 *
 * @param View The string to split.
 * @param Search The strings to search for and split on.
 * @param OutLeft Receives the part of the view to the left of the search string if it is found.
 * @param OutRight Receives the part of the view to the right of the search string if it is found.
 * @param SearchCase Whether the comparison should ignore case.
 * @param OutMatchIndex Receives the index of the search string that matched.
 * @return true if the search string was found and the outputs were written, otherwise false.
 */
CORE_API bool SplitLastOfAny(FAnsiStringView View, TConstArrayView<FAnsiStringView> Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API bool SplitLastOfAny(FUtf8StringView View, TConstArrayView<FUtf8StringView> Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API bool SplitLastOfAny(FWideStringView View, TConstArrayView<FWideStringView> Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);

/**
 * Split the view on the first occurrence of the search character.
 *
 * @param View The string to split.
 * @param Search The character to search for and split on.
 * @param OutLeft Receives the part of the view to the left of the search character if it is found.
 * @param OutRight Receives the part of the view to the right of the search character if it is found.
 * @param SearchCase Whether the comparison should ignore case.
 * @return true if the search character was found and the outputs were written, otherwise false.
 */
CORE_API bool SplitFirstChar(FAnsiStringView View, ANSICHAR Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);
CORE_API bool SplitFirstChar(FUtf8StringView View, UTF8CHAR Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);
CORE_API bool SplitFirstChar(FWideStringView View, WIDECHAR Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);

/**
 * Split the view on the last occurrence of the search character.
 *
 * @param View The string to split.
 * @param Search The character to search for and split on.
 * @param OutLeft Receives the part of the view to the left of the search character if it is found.
 * @param OutRight Receives the part of the view to the right of the search character if it is found.
 * @param SearchCase Whether the comparison should ignore case.
 * @return true if the search character was found and the outputs were written, otherwise false.
 */
CORE_API bool SplitLastChar(FAnsiStringView View, ANSICHAR Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);
CORE_API bool SplitLastChar(FUtf8StringView View, UTF8CHAR Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);
CORE_API bool SplitLastChar(FWideStringView View, WIDECHAR Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive);

/**
 * Split the view on the first occurrence of any search character.
 *
 * @param View The string to split.
 * @param Search The characters to search for and split on.
 * @param OutLeft Receives the part of the view to the left of the search character if it is found.
 * @param OutRight Receives the part of the view to the right of the search character if it is found.
 * @param SearchCase Whether the comparison should ignore case.
 * @param OutMatchIndex Receives the index of the search character that matched.
 * @return true if the search character was found and the outputs were written, otherwise false.
 */
CORE_API bool SplitFirstOfAnyChar(FAnsiStringView View, TConstArrayView<ANSICHAR> Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API bool SplitFirstOfAnyChar(FUtf8StringView View, TConstArrayView<UTF8CHAR> Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API bool SplitFirstOfAnyChar(FWideStringView View, TConstArrayView<WIDECHAR> Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);

/**
 * Split the view on the last occurrence of any search character.
 *
 * @param View The string to split.
 * @param Search The characters to search for and split on.
 * @param OutLeft Receives the part of the view to the left of the search character if it is found.
 * @param OutRight Receives the part of the view to the right of the search character if it is found.
 * @param SearchCase Whether the comparison should ignore case.
 * @param OutMatchIndex Receives the index of the search character that matched.
 * @return true if the search character was found and the outputs were written, otherwise false.
 */
CORE_API bool SplitLastOfAnyChar(FAnsiStringView View, TConstArrayView<ANSICHAR> Search, FAnsiStringView& OutLeft, FAnsiStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API bool SplitLastOfAnyChar(FUtf8StringView View, TConstArrayView<UTF8CHAR> Search, FUtf8StringView& OutLeft, FUtf8StringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);
CORE_API bool SplitLastOfAnyChar(FWideStringView View, TConstArrayView<WIDECHAR> Search, FWideStringView& OutLeft, FWideStringView& OutRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive, int32* OutMatchIndex = nullptr);

} // UE::String
