// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Fundamental/Scheduler.h"
#include "Async/Fundamental/Task.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Tasks/Task.h"
#include "Async/EventCount.h"
#include "Tasks/TaskPrivate.h"
#include "Templates/Invoke.h"
#include "Templates/UnrealTemplate.h"

#include <atomic>
#include <type_traits>

namespace UE::Tasks
{
	// A chain of tasks that are executed one after another. Can be used to synchronise access to a shared resource as FPipe guarantees 
	// non-concurrent tasks execution. FPipe is a replacement for named threads because it's lightweight and flexible -
	// there can be a large dynamic number of pipes each controlling its own shared resource. Can be used as a replacement for
	// dedicated threads.
	// Execution order is FIFO for tasks that don't have prerequisites, i.e. it's the same as launching order. 
	// Adding prerequisites to a pipe task can alter when the task is queued to the pipe, hence can change the execution order.
	// A pipe must be alive until its last task is completed.
	// See `FTasksPipeTest` for tests and examples.
	class FPipe
	{
	public:
		UE_NONCOPYABLE(FPipe);

		// @param InDebugName helps to identify the pipe in debugger and profiler. `UE_SOURCE_LOCATION` can be used as an auto-generated
		// unique name.
		explicit FPipe(const TCHAR* InDebugName)
			: EmptyEventRef(MakeShared<UE::FEventCount>())
			, DebugName(InDebugName)
		{}

		~FPipe()
		{
			check(!HasWork());
		}

		// returns `true` if the pipe has any not completed tasks
		bool HasWork() const
		{
			return TaskCount.load(std::memory_order_relaxed) != 0;
		}

		// waits until the pipe is empty (its last task is executed)
		// should be used only after no more tasks are launched in the pipe, e.g. preparing for the pipe destruction
		CORE_API bool WaitUntilEmpty(FTimespan Timeout = FTimespan::MaxValue());

		// launches a task in the pipe
		// @param InDebugName helps to identify the task in debugger and profiler
		// @param TaskBody a callable with no parameters, usually a lambda but can be also a functor object 
		// or a pointer to a function. TaskBody can return results.
		// @Priority - task priority, can affect task scheduling once it's passed the pipe
		// @param TaskFlags - task config options
		// @return Task instance that can be used to wait for task completion or to obtain the result of task execution
		template<typename TaskBodyType>
		TTask<TInvokeResult_T<TaskBodyType>> Launch
		(
			const TCHAR* InDebugName, 
			TaskBodyType&& TaskBody, 
			ETaskPriority Priority = ETaskPriority::Default,
			EExtendedTaskPriority ExtendedPriority = EExtendedTaskPriority::None,
			ETaskFlags Flags = ETaskFlags::None
		)
		{
			using FResult = TInvokeResult_T<TaskBodyType>;
			using FExecutableTask = Private::TExecutableTask<std::decay_t<TaskBodyType>>;

			FExecutableTask* Task = FExecutableTask::Create(InDebugName, Forward<TaskBodyType>(TaskBody), Priority, ExtendedPriority, Flags);
			TaskCount.fetch_add(1, std::memory_order_acq_rel);
			Task->SetPipe(*this);
			Task->TryLaunch(sizeof(*Task));
			return TTask<FResult>{ Task };
		}

		// launches a task in the pipe, with multiple prerequisites that must be completed before the task is scheduled
		// @param InDebugName helps to identify the task in debugger and profiler
		// @param TaskBody a callable with no parameters, usually a lambda but can be also a functor object 
		// or a pointer to a function. TaskBody can return results.
		// @Priority - task priority, can affect task scheduling once it's passed the pipe
		// @param TaskFlags - task config options
		// @return Task instance that can be used to wait for task completion or to obtain the result of task execution
		template<typename TaskBodyType, typename PrerequisitesCollectionType>
		TTask<TInvokeResult_T<TaskBodyType>> Launch
		(
			const TCHAR* InDebugName, 
			TaskBodyType&& TaskBody, 
			PrerequisitesCollectionType&& Prerequisites, 
			ETaskPriority Priority = ETaskPriority::Default,
			EExtendedTaskPriority ExtendedPriority = EExtendedTaskPriority::None,
			ETaskFlags Flags = ETaskFlags::None
		)
		{
			using FResult = TInvokeResult_T<TaskBodyType>;
			using FExecutableTask = Private::TExecutableTask<std::decay_t<TaskBodyType>>;

			FExecutableTask* Task = FExecutableTask::Create(InDebugName, Forward<TaskBodyType>(TaskBody), Priority, ExtendedPriority, Flags);
			TaskCount.fetch_add(1, std::memory_order_acq_rel);

			// Order matters here, pipe must be set before prerequisites try to unlock us
			// otherwise we could race between SetPipe and TryUnlock.
			// Pipe and NumLock must both be consistent together at the time of Unlock.
			Task->SetPipe(*this);
			Task->AddPrerequisites(Forward<PrerequisitesCollectionType>(Prerequisites));
			Task->TryLaunch(sizeof(*Task));
			return TTask<FResult>{ Task };
		}

		// checks if pipe's task is being executed by the current thread. Allows to check if accessing a resource protected by a pipe
		// is thread-safe
		CORE_API bool IsInContext() const;

	private:
		friend class Private::FTaskBase;

		// pushes given task into the pipe: adds the task as a subsequent to the last task if any and sets it as the new last task
		// returns the accounted reference to the previous piped task if we managed to register the given task as its subsequent, otherwise nullptr.
		// the reference must be released by the caller when not needed anymore
		[[nodiscard]] Private::FTaskBase* PushIntoPipe(Private::FTaskBase& Task);

		// pipe holds a "weak" reference to a task. the task must be cleared from the pipe when its execution finished before its completion, 
		// otherwise the next piped task can try to add itself as a subsequent to an already destroyed task.
		void ClearTask(Private::FTaskBase& Task);

		// notifications about pipe's task execution
		void ExecutionStarted();
		void ExecutionFinished();
	private:
		// pipe builds a chain (a linked list) of tasks and so needs to store only the last one. the last task is null if the pipe is not blocked
		std::atomic<Private::FTaskBase*> LastTask{ nullptr };
		std::atomic<uint64> TaskCount { 0 };
		TSharedRef<UE::FEventCount> EmptyEventRef;
	public:
		FORCENOINLINE const TCHAR* GetDebugName() const
		{
			return DebugName;
		}

	private:
		const TCHAR* const DebugName;
	};
}
