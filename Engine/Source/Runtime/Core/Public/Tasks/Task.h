// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/TaskPrivate.h"
#include "Async/Fundamental/Task.h"
#include "Async/ManualResetEvent.h"
#include "Containers/StaticArray.h"
#include "HAL/Event.h"
#include "HAL/IConsoleManager.h"
#include "CoreTypes.h"

/** 
 * This namespace provides utilities for creating and accessing the state of tasks that run on separate threads.
 * You can do this by using one of the Launch functions to start a task and return a handle for later use. For example:
 *
 *		UE::Tasks::TTask<FReturnValue> TaskHandle = UE::Tasks::Launch(TEXT("TaskDebugName"), []() {
 *			DoSomethingSlow();
 *			return FReturnValue();
 *		});
 * 
 * In this example, the task will run on any worker thread and return a result payload that can be accessed
 * on the calling thread when the task has completed. Tasks can be launched with a list of prerequisites,
 * which will delay execution of the task until the prerequisites have completed. UE::Tasks::FTaskEvent can 
 * be used as a prerequisite to support manual triggering of task execution.
 * 
 * Read the full documentation for an explanation of the different parameters to the launch function and
 * the different ways to use the task handle after it has been launched.
 */
namespace UE::Tasks
{

	namespace Private
	{
		/** Base implementation used by TTask to provide handle functionality */
		class FTaskHandle
		{
		public:
			using FTaskHandleId = void;

			FTaskHandle() = default;

			/** True if the task has been correctly initialized and not yet destroyed */
			bool IsValid() const
			{
				return Pimpl.IsValid();
			}

			/** Checks if task's execution is done */
			bool IsCompleted() const
			{
				return !IsValid() || Pimpl->IsCompleted();
			}

			/**
			 * Waits for task's completion with timeout. Tries to retract the task and execute it in-place, if failed - blocks until the task 
			 * is completed by another thread. If timeout is zero, tries to retract the task and returns immedially after that.
			 * @return true if the task is completed
			 */
			bool Wait(FTimespan Timeout) const
			{
				return !IsValid() || Pimpl->Wait(FTimeout{ Timeout });
			}

			/**
			 * Waits for task's completion without timeout. Tries to retract the task and execute it in-place, if failed - blocks until the task 
			 * is completed by another thread.
			 * @return true if the task is completed
			 */
			bool Wait() const
			{
				if (IsValid())
				{
					Pimpl->Wait();
				}
				
				// we keep the return type as boolean even if always returning true
				// as we could merge some versions of the Wait API back at some point.
				return true;
			}

			/**
			 * Try to finish the task on the current thread if it's not already started.
			 * @return true if the task is completed
			 */
			bool TryRetractAndExecute()
			{
				Pimpl->TryRetractAndExecute(FTimeout::Never());
				return IsCompleted();
			}

			/**
			 * Launches a task for asynchronous execution
			 * @param DebugName - a unique name for task identification in debugger and profiler, is compiled out in test/shipping builds
			 * @param TaskBody - a functor that will be executed asynchronously
			 * @param Priority - task priority that affects when the task will be executed
			 * @param ExtendedPriority - priority options for integrating with other system like named threads
	 		 * @param Flags - task config options
			 * @return a trivially relocatable instance that can be used to wait for task completion or to obtain task execution result
			 */
			template<typename TaskBodyType>
			void Launch(
				const TCHAR* DebugName,
				TaskBodyType&& TaskBody,
				ETaskPriority Priority = ETaskPriority::Normal,
				EExtendedTaskPriority ExtendedPriority = EExtendedTaskPriority::None,
				ETaskFlags Flags = ETaskFlags::None)
			{
				check(!IsValid());

				using FExecutableTask = Private::TExecutableTask<std::decay_t<TaskBodyType>>;
				FExecutableTask* Task = FExecutableTask::Create(DebugName, Forward<TaskBodyType>(TaskBody), Priority, ExtendedPriority, Flags);
				// this must happen before launching, to support an ability to access the task itself from inside it
				*Pimpl.GetInitReference() = Task;
				Task->TryLaunch(sizeof(*Task));
			}

			/**
			 * Launches a task for asynchronous execution, with prerequisites that must be completed before the task is scheduled
			 * @param DebugName - a unique name for task identification in debugger and profiler, is compiled out in test/shipping builds
			 * @param TaskBody - a functor that will be executed asynchronously
			 * @param Prerequisites - tasks or task events that must be completed before the task being launched can be scheduled, accepts any 
			 *		iterable collection (.begin()/.end()), `Tasks::Prerequisites()` helper is recommended to create such collection on the fly
			 * @param Priority - task priority that affects when the task will be executed
			 * @param ExtendedPriority - priority options for integrating with other system like named threads
			 * @param Flags - task config options
			 * @return a trivially relocatable instance that can be used to wait for task completion or to obtain task execution result
			 */
			template<typename TaskBodyType, typename PrerequisitesCollectionType>
			void Launch(
				const TCHAR* DebugName,
				TaskBodyType&& TaskBody,
				PrerequisitesCollectionType&& Prerequisites,
				ETaskPriority Priority = ETaskPriority::Normal,
				EExtendedTaskPriority ExtendedPriority = EExtendedTaskPriority::None,
				ETaskFlags Flags = ETaskFlags::None)
			{
				check(!IsValid());

				using FExecutableTask = Private::TExecutableTask<std::decay_t<TaskBodyType>>;
				FExecutableTask* Task = FExecutableTask::Create(DebugName, Forward<TaskBodyType>(TaskBody), Priority, ExtendedPriority, Flags);
				Task->AddPrerequisites(Forward<PrerequisitesCollectionType>(Prerequisites));
				// this must happen before launching, to support an ability to access the task itself from inside it
				*Pimpl.GetInitReference() = Task;
				Task->TryLaunch(sizeof(*Task));
			}

			/** Returns true if it is valid to Wait on this task */
			bool IsAwaitable() const
			{
				return IsValid() && Pimpl->IsAwaitable();
			}

			/** Gets the priority this task was initialized with */
			ETaskPriority GetPriority() const
			{
				return Pimpl->GetPriority();
			}

			/** Gets the extended priority this task was initialized with */
			EExtendedTaskPriority GetExtendedPriority() const
			{
				return Pimpl->GetExtendedPriority();
			}

			bool operator==(const FTaskHandle& Other) const
			{
				return Pimpl == Other.Pimpl;
			}

			bool operator!=(const FTaskHandle& Other) const
			{
				return Pimpl != Other.Pimpl;
			}
			
		private:
			// friends to get access to `Pimpl`
			friend FTaskBase;

			template<int32 Index, typename ArrayType, typename FirstTaskType, typename... OtherTasksTypes>
			friend void PrerequisitesUnpacker(ArrayType& Array, FirstTaskType& FirstTask, OtherTasksTypes&... OtherTasks);

			template<int32 Index, typename ArrayType, typename TaskType>
			friend void PrerequisitesUnpacker(ArrayType& Array, TaskType& FirstTask);

			template<typename TaskCollectionType>
			friend bool TryRetractAndExecute(const TaskCollectionType& Tasks, FTimeout Timeout);

			template<typename TaskCollectionType>
			friend TArray<TaskTrace::FId> GetTraceIds(const TaskCollectionType& Tasks);

			template<typename TaskCollectionType>
			friend bool UE::Tasks::Wait(const TaskCollectionType& Tasks, FTimespan InTimeout);

			template<typename TaskType>
			friend void UE::Tasks::AddNested(const TaskType& Nested);

		protected:
			explicit FTaskHandle(FTaskBase* Other)
				: Pimpl(Other, /*bAddRef = */false)
			{}

			TRefCountPtr<FTaskBase> Pimpl;
		};
	}

	/** Generic task type, can be either a real task or event */
	using FTask = Private::FTaskHandle;

	/** A templated asynchronous task that returns a result payload when complete */
	template<typename ResultType>
	class TTask : public Private::FTaskHandle
	{
		// a movable/copyable handle of `Private::TTaskWithResult*` with the API adopted for public usage.
		// implements Pimpl idiom

	public:
		TTask() = default;

		/** Waits until the task is completed and returns task's result */
		ResultType& GetResult()
		{
			check(IsValid());
			FTaskHandle::Wait();
			return static_cast<Private::TTaskWithResult<ResultType>*>(Pimpl.GetReference())->GetResult();
		}

	private:
		friend FPipe;

		// private constructor, valid instances can be created only by launching (see friends)
		explicit TTask(Private::FTaskBase* Other)
			: FTaskHandle(Other)
		{}
	};

	/** A task that does not return a result payload */
	template<>
	class TTask<void> : public Private::FTaskHandle
	{
	public:
		TTask() = default;

		/** Waits until task is completed, there is no result payload */
		void GetResult()
		{
			check(IsValid()); // to be consistent with a generic `TTask<ResultType>::GetResult()`
			Wait();
		}

	private:
		friend FPipe;

		// private constructor, valid instances can be created only by launching (see friends)
		explicit TTask(Private::FTaskBase* Other)
			: FTaskHandle(Other)
		{}
	};

	/** 
	 * A synchronisation primitive, a recommended substitution of `FEvent` for signalling between tasks. If used as a task prerequisite or 
	 * a nested task, it doesn't block a worker thread. Optionally can use "busy waiting" - executing tasks while waiting.
	 */
	class FTaskEvent : public Private::FTaskHandle
	{
	public:
		explicit FTaskEvent(const TCHAR* DebugName)
			: Private::FTaskHandle(Private::FTaskEventBase::Create(DebugName))
		{
		}

		/** Add tasks to wait on, all prerequisites must be added before triggering the event */
		template<typename PrerequisitesType>
		void AddPrerequisites(const PrerequisitesType& Prerequisites)
		{
			Pimpl->AddPrerequisites(Prerequisites);
		}

		/** Activates the task event, which will complete if the prerequisites have completed */
		void Trigger()
		{
			if (!IsCompleted()) // event can be triggered multiple times
			{
				Pimpl->Trigger(sizeof(*Pimpl));
			}
		}
	};

	/**
	 * Launches a task for asynchronous execution
	 * @param DebugName - a unique name for task identification in debugger and profiler, is compiled out in test/shipping builds
	 * @param TaskBody - a functor that will be executed asynchronously
	 * @param Priority - task priority that affects when the task will be executed
	 * @param ExtendedPriority - priority options for integrating with other system like named threads
	 * @param Flags - task config options
	 * @return a trivially relocatable instance that can be used to wait for task completion or to obtain task execution result
	 */
	template<typename TaskBodyType>
	TTask<TInvokeResult_T<TaskBodyType>> Launch(
		const TCHAR* DebugName,
		TaskBodyType&& TaskBody,
		ETaskPriority Priority = ETaskPriority::Normal,
		EExtendedTaskPriority ExtendedPriority = EExtendedTaskPriority::None,
		ETaskFlags Flags = ETaskFlags::None)
	{
		using FResult = TInvokeResult_T<TaskBodyType>;
		TTask<FResult> Task;
		Task.Launch(DebugName, Forward<TaskBodyType>(TaskBody), Priority, ExtendedPriority, Flags);
		return Task;
	}

	/**
	 * Launches a task for asynchronous execution, with prerequisites that must be completed before the task is scheduled
	 * @param DebugName - a unique name for task identification in debugger and profiler, is compiled out in test/shipping builds
	 * @param TaskBody - a functor that will be executed asynchronously
	 * @param Prerequisites - tasks or task events that must be completed before the task being launched can be scheduled, accepts any 
	 *		iterable collection (.begin()/.end()), `Tasks::Prerequisites()` helper is recommended to create such collection on the fly
	 * @param Priority - task priority that affects when the task will be executed
	 * @param ExtendedPriority - priority options for integrating with other system like named threads
	 * @param Flags - task config options
	 * @return a trivially relocatable instance that can be used to wait for task completion or to obtain task execution result
	 */
	template<typename TaskBodyType, typename PrerequisitesCollectionType>
	TTask<TInvokeResult_T<TaskBodyType>> Launch(
		const TCHAR* DebugName,
		TaskBodyType&& TaskBody,
		PrerequisitesCollectionType&& Prerequisites,
		ETaskPriority Priority = ETaskPriority::Normal,
		EExtendedTaskPriority ExtendedPriority = EExtendedTaskPriority::None,
		ETaskFlags Flags = ETaskFlags::None)
	{
		using FResult = TInvokeResult_T<TaskBodyType>;
		TTask<FResult> Task;
		Task.Launch(DebugName, Forward<TaskBodyType>(TaskBody), Forward<PrerequisitesCollectionType>(Prerequisites), Priority, ExtendedPriority, Flags);
		return Task;
	}

	namespace Private
	{
		template<int32 Index, typename ArrayType, typename FirstTaskType, typename... OtherTasksTypes>
		void PrerequisitesUnpacker(ArrayType& Array, FirstTaskType& FirstTask, OtherTasksTypes&... OtherTasks)
		{
			Array[Index] = FirstTask.Pimpl.GetReference();
			PrerequisitesUnpacker<Index + 1>(Array, OtherTasks...);
		}

		template<int32 Index, typename ArrayType, typename TaskType>
		void PrerequisitesUnpacker(ArrayType& Array, TaskType& Task)
		{
			Array[Index] = Task.Pimpl.GetReference();
		}

		template<typename HigherLevelTaskType, std::enable_if_t<std::is_same_v<HigherLevelTaskType, FTask>>* = nullptr>
		bool IsCompleted(const HigherLevelTaskType& Prerequisite)
		{
			return Prerequisite.IsCompleted();
		}

		template<typename HigherLevelTaskType, std::enable_if_t<std::is_same_v<HigherLevelTaskType, FGraphEventRef>>* = nullptr>
		bool IsCompleted(const HigherLevelTaskType& Prerequisite)
		{
			return Prerequisite.IsValid() ? Prerequisite->IsCompleted() : false;
		}
	}

	/** Returns prerequisites for an array of tasks */
	template<typename... TaskTypes, 
		typename std::decay_t<decltype(std::declval<TTuple<TaskTypes...>>().template Get<0>())>::FTaskHandleId* = nullptr>
	TStaticArray<Private::FTaskBase*, sizeof...(TaskTypes)> Prerequisites(TaskTypes&... Tasks)
	{
		TStaticArray<Private::FTaskBase*, sizeof...(TaskTypes)> Res;
		Private::PrerequisitesUnpacker<0>(Res, Tasks...);
		return Res;
	}

	template<typename TaskCollectionType>
	const TaskCollectionType& Prerequisites(const TaskCollectionType& Tasks)
	{
		return Tasks;
	}

	/** Utility function to wait on a specific task */
	inline void Wait(FTask& Task)
	{
		Task.Wait();
	}

	/**
	 * Wait for multiple tasks, with timeout
	 * @param TaskCollectionType - an iterable collection of `TTask<T>`, e.g. `TArray<FTask>`
	 */
	template<typename TaskCollectionType>
	bool Wait(const TaskCollectionType& Tasks, FTimespan InTimeout/* = FTimespan::MaxValue()*/)
	{
		TaskTrace::FWaitingScope WaitingScope(Private::GetTraceIds(Tasks));
		TRACE_CPUPROFILER_EVENT_SCOPE(Tasks::Wait);

		return UE::Tasks::Launch(
				TEXT("Waiting Task"),
				[](){},
				UE::Tasks::Prerequisites(Tasks),
				ETaskPriority::Default /* doesn't matter */,
				EExtendedTaskPriority::Inline
			).Wait(InTimeout);
	}

	/////////////////////////////////////////////////////////////
	// "any task" support. these functions allocate excessively (per input task plus more).
	// can be reduced to a single alloc if this is a perf issue
	
	/**
	 * Blocks the current thread until any of the given tasks is completed.
	 * Is slightly more efficient than `Any().Wait()`.
	 * Returns the index of the first completed task, or `INDEX_NONE` on timeout
	 */
	template<typename TaskCollectionType>
	int32 WaitAny(const TaskCollectionType& Tasks, FTimespan Timeout = FTimespan::MaxValue())
	{
		if (UNLIKELY(Tasks.Num() == 0))
		{
			return INDEX_NONE;
		}

		// Avoid memory allocations if any of the events are already completed
		for (int32 Index = 0; Index < Tasks.Num(); ++Index)
		{
			if (Private::IsCompleted(Tasks[Index]))
			{
				return Index;
			}
		}

		struct FSharedData
		{
			UE::FManualResetEvent Event;
			std::atomic<int32>    CompletedTaskIndex{ 0 };
		};

		// Shared data usage is important to avoid the variable to go out of scope
		// before all the task have been run even if we exit after the first event
		// is triggered.
		TSharedRef<FSharedData> SharedData = MakeShared<FSharedData>();

		for (int32 Index = 0; Index < Tasks.Num(); ++Index)
		{
			Launch(UE_SOURCE_LOCATION, 
				[SharedData, Index]
				{ 
					SharedData->CompletedTaskIndex.store(Index, std::memory_order_relaxed);
					SharedData->Event.Notify();
				}, 
				Prerequisites(Tasks[Index]),
				ETaskPriority::Default, 
				EExtendedTaskPriority::Inline
			);
		}

		if (SharedData->Event.WaitFor(UE::FMonotonicTimeSpan::FromMilliseconds(Timeout.GetTotalMilliseconds())))
		{
			return SharedData->CompletedTaskIndex.load(std::memory_order_relaxed);
		}

		return INDEX_NONE;
	}

	/** Returns a task that gets completed as soon as any of the given tasks gets completed */
	template<typename TaskCollectionType>
	FTask Any(const TaskCollectionType& Tasks)
	{
		if (UNLIKELY(Tasks.Num() == 0))
		{
			return FTask{};
		}

		struct FSharedData
		{
			explicit FSharedData(uint32 InitRefCount)
				: RefCount(InitRefCount)
			{
			}

			FTaskEvent Event{ UE_SOURCE_LOCATION };
			std::atomic<uint32> RefCount;
		};

		FSharedData* SharedData = new FSharedData(Tasks.Num());
		// `SharedData` can be destroyed before leaving the scope, cache the result locally
		FTaskEvent Result = SharedData->Event;

		for (const FTask& Task : Tasks)
		{
			Launch(UE_SOURCE_LOCATION,
				[SharedData, Num = Tasks.Num()]
				{
					// cache the local copy as `SharedData` can be concurrently deleted right after decrementing the ref counter
					FTaskEvent Event = SharedData->Event;
					uint32 PrevRefCount = SharedData->RefCount.fetch_sub(1, std::memory_order_acq_rel); // acq_rel to sync between tasks

					if (UNLIKELY(PrevRefCount == Num))
					{	// the first completed task
						Event.Trigger();
					}

					if (UNLIKELY(PrevRefCount == 1))
					{	// the last completed task
						delete SharedData;
					}
				},
				Prerequisites(Task),
				ETaskPriority::Default,
				EExtendedTaskPriority::Inline
			);
		}

		return Result;
	}

	/**
	 * Adds the nested task to the task that is being currently executed by the current thread. A parent task is not flagged completed
	 * until all nested tasks are completed. It's similar to explicitly waiting for a sub-task at the end of its parent task, except explicit waiting
	 * blocks the worker executing the parent task until the sub-task is completed. With nested tasks, the worker won't be blocked.
	 */
	template<typename TaskType>
	void AddNested(const TaskType& Nested)
	{
		Private::FTaskBase* Parent = Private::GetCurrentTask();
		check(Parent != nullptr);
		Parent->AddNested(*Nested.Pimpl);
	}

	/** Creates and returns an already completed task holding a result value constructed from given args */
	template<typename ResultType, typename... ArgTypes>
	TTask<ResultType> MakeCompletedTask(ArgTypes&&... Args)
	{
		return Launch(
			UE_SOURCE_LOCATION,
			[&] { return ResultType(Forward<ArgTypes>(Args)...); },
			ETaskPriority::Default, // doesn't matter
			EExtendedTaskPriority::Inline);
	}

	/**
	 * Console variable for configuring task priorities
	 * Example:
	 *		FTaskPriorityCVar CVar{ TEXT("CVarName"), TEXT("CVarHelp"), ETaskPriority::Normal, EExtendedTaskPriority::None };
	 *		Launch(UE_SOURCE_LOCATION, [] {}, CVar.GetTaskPriority(), CVar.GetExtendedTaskPriority()).Wait();
	 */
	class FTaskPriorityCVar
	{
	public:
		CORE_API FTaskPriorityCVar(const TCHAR* Name, const TCHAR* Help, ETaskPriority DefaultPriority, EExtendedTaskPriority DefaultExtendedPriority);

		ETaskPriority GetTaskPriority() const
		{
			return Priority;
		}

		EExtendedTaskPriority GetExtendedTaskPriority() const
		{
			return ExtendedPriority;
		}

	private:
		static FString CreateFullHelpText(const TCHAR* Name, const TCHAR* OriginalHelp);
		static FString ConfigStringFromPriorities(ETaskPriority InPriority, EExtendedTaskPriority InExtendedPriority);
		void OnSettingChanged(IConsoleVariable* Variable);

	private:
		FString RawSetting;
		FString FullHelpText;
		FAutoConsoleVariableRef Variable;
		ETaskPriority Priority;
		EExtendedTaskPriority ExtendedPriority;
	};

	/**
	 * Support for canceling tasks mid-execution
	 * usage:
	 *		FCancellationToken Token;
	 *		Launch(UE_SOURCE_LOCATION, [&Token] { ... if (Token.IsCanceled()) return; ... });
	 *		Token.Cancel();
	 * it's user's decision and responsibility to manage cancellation token lifetime, to check cancellation token, return early, 
	 *		to do any required cleanup, or to do nothing at all
	 * no way to cancel a task to skip its execution completely
	 * waiting for a canceled task is blocking until its prerequisites are completed and the task is executed and completed, 
	 *		basically same as for not canceled tasks except a canceled one can quit execution early
	 * canceling a task doesn't affect its subsequents (unless they use the same cancellation token instance)
	 */
	class FCancellationToken
	{
	public:
		UE_NONCOPYABLE(FCancellationToken);
		FCancellationToken() = default;

		void Cancel()
		{
			bCanceled.store(true, std::memory_order_relaxed);
		}

		bool IsCanceled() const
		{
			return bCanceled.load(std::memory_order_relaxed);
		}

	private:
		std::atomic<bool> bCanceled{ false };
	};

	class FCancellationTokenScope
	{
	public:
		CORE_API FCancellationTokenScope(FCancellationToken& CancellationToken);
		CORE_API FCancellationTokenScope(FCancellationToken* CancellationToken);
		CORE_API ~FCancellationTokenScope();

		CORE_API static FCancellationToken* GetCurrentCancellationToken();

		static bool IsCurrentWorkCanceled()
		{
			if (FCancellationToken* CurrentCancellationToken = GetCurrentCancellationToken())
			{
				return CurrentCancellationToken->IsCanceled();
			}

			return false;
		}
	private:
		bool bHasActiveScope = false;
		void SetToken(FCancellationToken* CancellationToken);
	};
}
