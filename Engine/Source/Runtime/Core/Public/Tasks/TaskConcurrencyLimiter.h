// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/Task.h"
#include "Containers/LockFreeList.h"
#include "Templates/SharedPointer.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "CoreTypes.h"

#include <AtomicQueue.h>

namespace UE::Tasks::TaskConcurrencyLimiter_Private
{

// a queue of free slots in range [0 .. max_concurrency). Initially contains all slots in the range.
class FConcurrencySlots
{
public:
	explicit FConcurrencySlots(uint32 MaxConcurrency)
		: FreeSlots(MaxConcurrency)
	{
		for (uint32 Index = IndexOffset; Index < MaxConcurrency + IndexOffset; ++Index)
		{
			FreeSlots.push(Index);
		}
	}

	bool Alloc(uint32& Slot)
	{
		if (FreeSlots.try_pop(Slot))
		{
			Slot -= IndexOffset;
			return true;
		}

		return false;
	}

	void Release(uint32 Slot)
	{
		FreeSlots.push(Slot + IndexOffset);
	}

private:
	// this queue uses 0 as a special "null" value. to work around this, slots are shifted by one for storage, thus ending up in 
	// [1 .. max_concurrency] range
	static constexpr int32 IndexOffset = 1;
	atomic_queue::AtomicQueueB<uint32> FreeSlots; // a bounded lock-free FIFO queue
};

// an implementation details of FTaskConcurrenctyLimiter
class FPimpl : public TSharedFromThis<FPimpl>
{
	// Reference counted wrapper for FTask to manage the task lifetime between worker threads and 
	// any thread that might retract the task once it is submitted to the scheduler
	struct FLimiterTask : FRefCountedObject
	{
		LowLevelTasks::FTask Task;
	};

public:
	explicit FPimpl(uint32 InMaxConcurrency, ETaskPriority InTaskPriority)
		: ConcurrencySlots(InMaxConcurrency)
		, TaskPriority(InTaskPriority)
	{
		ScheduledTasks.SetNum(InMaxConcurrency);
	}

	CORE_API ~FPimpl();

	template<typename TaskFunctionType>
	void Push(const TCHAR* DebugName, TaskFunctionType&& TaskFunction);

	bool CORE_API Wait(FTimespan Timeout);

private:
	void CORE_API AddWorkItem(FLimiterTask* Task);
	void CORE_API ProcessQueue(uint32 ConcurrencySlot, bool bSkipFirstWakeUp);
	void CORE_API ProcessQueueFromWorker(uint32 ConcurrencySlot);
	void CORE_API ProcessQueueFromPush(uint32 ConcurrencySlot);
	void CORE_API CompleteWorkItem(uint32 ConcurrencySlot);

	// Padded struct to prevent false sharing as we read and write to the ScheduledTasks array			
	struct alignas(PLATFORM_CACHE_LINE_SIZE) FPaddedSharedTask
	{
		std::atomic<FLimiterTask*> Task;
	};

	FConcurrencySlots ConcurrencySlots; // free slots queue. used also to limit concurrency
	TLockFreePointerListFIFO<FLimiterTask, PLATFORM_CACHE_LINE_SIZE> WorkQueue; // a queue of user-provided task functions
	TArray<FPaddedSharedTask> ScheduledTasks;
	std::atomic<FEvent*> CompletionEvent { nullptr };
	std::atomic<uint32> NumWorkItems { 0 };
	ETaskPriority TaskPriority;
};

///////////////////////////////////////////////////////
// FPimpl's template implementations
///////////////////////////////////////////////////////

template<typename TaskFunctionType>
void FPimpl::Push(const TCHAR* DebugName, TaskFunctionType&& TaskFunction)
{
	FLimiterTask* LimiterTask = new FLimiterTask();
	LowLevelTasks::FTask& Task = LimiterTask->Task;
	Task.Init(
		DebugName,
		TaskPriority,
		[
			TaskFunction = MoveTemp(TaskFunction),
			this,
			// Assign Pimpl to keep the TaskConcurrenyLimiter alive as long as the Task we created is alive.
		Pimpl = TSharedFromThis<FPimpl>::AsShared(),
		// Assign FLimiterTask wrapper into a TRefCountPtr to keep the LimiterTask alive as long as its inner Task is alive.
		LimiterTask = TRefCountPtr<FLimiterTask>(LimiterTask)
		]()
		{
			LowLevelTasks::FTask& Task = LimiterTask->Task;

			// We can't pass the ConcurrencySlot in the lambda during creation as
			// it's not actually acquired yet. The value is assigned during ProcessQueue
			// after we have called Alloc and received a ConcurrencySlot for it.
			uint32 ConcurrencySlot = (uint32)(UPTRINT)Task.GetUserData();

			// This task is now executing, either on a taskthread that received the task we created, or on
			// the thread that called Wait if the waitthread reached the TryExpedite call before the
			// taskthread started the Task.
			// In the case that we are in the taskthread, we need to implement our two contracts:
			//    1. Remove Task from ScheduledTasks as soon as it starts.
			//		 This prevent waitthread wastefully calling TryExpedite in most cases.
			//           Note a useless TryExpedite will still occur and be a noop if the threads are racing.
			//       Remove the pointer so that we can later safely free the Tasks's memory.
			//    2. Call Release on Task when removing it from ScheduledTasks.
			//       This balances AddRef done in ProcessQueue.
			// Note that if we reach here in the waitthread, then we have already removed the task from
			// ScheduledTasks and this call to exchange is a noop and we set bOwnLimiterTask=false.
			bool bOwnLimiterTask = ScheduledTasks[ConcurrencySlot].Task.exchange(nullptr, std::memory_order_release) != nullptr;
			if (bOwnLimiterTask)
			{
				// Whomever obtains the task from ScheduledTasks must release the reference
				LimiterTask->Release();
			}

			TaskFunction(ConcurrencySlot);

			// Finish bookkeeping for the current task and look for more tasks to launch.
			// If we find another task to launch, have it use the ConcurrencySlot we just finished
			// with; note this means use of a ConcurrencySlot may shift around between threads.
			// For the first task we find, bias it towards the current worker thread if possible.
			// If we find more than one, kick off a new Task for the global scheduler to schedule,
			// for every task after the first.
			CompleteWorkItem(ConcurrencySlot);
		}
	);

	AddWorkItem(LimiterTask);
}

} // namespace UE::Tasks::TaskConcurrencyLimiter_Private


namespace UE::Tasks
{

/**
* A lightweight construct that limits the concurrency of tasks pushed into it. 
*
* @note This class supports being destroyed before the tasks it contains are finished.
*/
class FTaskConcurrencyLimiter
{
public:
	/**
		* Constructor.
		*
		* @param MaxConcurrency     How wide the processing can go.
		* @param TaskPriority       Priority the tasks will be launched with.
		*/
	explicit FTaskConcurrencyLimiter(uint32 MaxConcurrency, ETaskPriority TaskPriority = ETaskPriority::Default)
		: Pimpl(MakeShared<TaskConcurrencyLimiter_Private::FPimpl>(MaxConcurrency, TaskPriority))
	{
	}

	/**
		* Push a new task.
		*
		* @param DebugName    Helps to identify the task in debugger and profiler.
		* @param TaskFunction A callable with a slot parameter, usually a lambda but can be also a functor object 
		*                     or a pointer to a function. The slot parameter is an index in [0..max_concurrency) range, 
		*                     unique at any moment of time, that can be used in user code to index a fixed-size buffer. 
		*                     See `TaskConcurrencyLimiterStressTest()` for an example.
		*/
	template<typename TaskFunctionType>
	void Push(const TCHAR* DebugName, TaskFunctionType&& TaskFunction)
	{
		Pimpl->Push(DebugName, MoveTemp(TaskFunction));
	}

	/**
		* Waits for task's completion with timeout.
		*
		* @param  Timeout Maximum amount of time to wait for tasks to finish before returning.
		* @return true if all tasks are completed, false otherwise.
		* 
		* @note   A wait is satisfied once the internal task counter reaches 0 and is never reset
		*         afterward when more tasks are added. A new FTaskConcurrencyLimiter can be used
		*         for such a use case.
		*/
	bool Wait(FTimespan Timeout = FTimespan::MaxValue())
	{
		return Pimpl->Wait(Timeout);
	}

private:
	TSharedRef<TaskConcurrencyLimiter_Private::FPimpl> Pimpl;
};

} // namespace UE::Tasks
