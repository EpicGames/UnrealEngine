// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/EventCount.h"
#include "Async/Fundamental/Scheduler.h"
#include "Async/Fundamental/Task.h"
#include "Async/Mutex.h"
#include "Async/TaskGraphFwd.h"
#include "Async/TaskTrace.h"
#include "Async/UniqueLock.h"
#include "Containers/Array.h"
#include "Containers/LockFreeFixedSizeAllocator.h"
#include "Containers/LockFreeList.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTLS.h"
#include "HAL/Thread.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Timeout.h"
#include "Misc/Timespan.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Templates/EnableIf.h"
#include "Templates/Invoke.h"
#include "Templates/MemoryOps.h"
#include "Templates/RefCounting.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "Async/InheritedContext.h"

#include <atomic>
#include <type_traits>

#ifndef WITH_TASKGRAPH_VERBOSE_TRACE
#define WITH_TASKGRAPH_VERBOSE_TRACE 0
#endif

#if WITH_TASKGRAPH_VERBOSE_TRACE
#define TASKGRAPH_VERBOSE_EVENT_SCOPE(Name) TRACE_CPUPROFILER_EVENT_SCOPE(Name)
#else
#define TASKGRAPH_VERBOSE_EVENT_SCOPE(Name)
#endif

namespace UE::Tasks
{
	using LowLevelTasks::ETaskPriority;
	using LowLevelTasks::ToString;
	using LowLevelTasks::ToTaskPriority;

	/** Options used to override general scheduler priority and integrate with other systems */
	enum class EExtendedTaskPriority : int8
	{
		None,
		Inline, // a task priority for "inline" task execution - a task is executed "inline" by the thread that unlocked it, w/o scheduling
		TaskEvent, // a task priority used by task events, allows to shortcut task execution


		// for integration with named threads
		GameThreadNormalPri,
		GameThreadHiPri,
		GameThreadNormalPriLocalQueue,
		GameThreadHiPriLocalQueue,

		RenderThreadNormalPri,
		RenderThreadHiPri,
		RenderThreadNormalPriLocalQueue,
		RenderThreadHiPriLocalQueue,

		RHIThreadNormalPri,
		RHIThreadHiPri,
		RHIThreadNormalPriLocalQueue,
		RHIThreadHiPriLocalQueue,

		Count
	};

	const TCHAR* ToString(EExtendedTaskPriority ExtendedPriority);
	bool ToExtendedTaskPriority(const TCHAR* ExtendedPriorityStr, EExtendedTaskPriority& OutExtendedPriority);

	enum class ETaskFlags
	{
		None,
		DoNotRunInsideBusyWait // do not pick this task for busy-waiting
	};

	class FPipe;

	template<typename ResultType>
	class TTask;

	template<typename TaskCollectionType>
	bool Wait(const TaskCollectionType& Tasks, FTimespan InTimeout = FTimespan::MaxValue());

	template<typename TaskType> void AddNested(const TaskType& Nested);

	/** Namespace for internals of task system, do not use these from general engine code */
	namespace Private
	{
		class FTaskBase;

		CORE_API void TranslatePriority(ENamedThreads::Type ThreadType, ETaskPriority& OutPriority, EExtendedTaskPriority& OutExtendedPriority);
		CORE_API ENamedThreads::Type TranslatePriority(ETaskPriority Priority, EExtendedTaskPriority ExtendedPriority);

		// returns the task (if any) that is being executed by the current thread
		CORE_API FTaskBase* GetCurrentTask();
		// sets the current task and returns the previous current task
		CORE_API FTaskBase* ExchangeCurrentTask(FTaskBase* Task);

		// Returns true if called from inside a task that is being retracted
		UE_DEPRECATED(5.1, "You should not use this function as it exists only to patch another system and can be removed any time.")
		CORE_API bool IsThreadRetractingTask();

		// An abstract base class for task implementation. 
		// Implements internal logic of task prerequisites, nested tasks and deep task retraction.
		// Implements intrusive ref-counting and so can be used with TRefCountPtr.
		// It doesn't store task body, instead it expects a derived class to provide a task body as a parameter to `TryExecute` method. @see TExecutableTask
		class FTaskBase : private UE::FInheritedContextBase
		{
			UE_NONCOPYABLE(FTaskBase);

			// `ExecutionFlag` is set at the beginning of execution as the most significant bit of `NumLocks` and indicates a switch 
			// of `NumLocks` from "execution prerequisites" (a number of uncompleted prerequisites that block task execution) to 
			// "completion prerequisites" (a number of nested uncompleted tasks that block task completion)
			static constexpr uint32 ExecutionFlag = 0x80000000;

			////////////////////////////////////////////////////////////////////////////
			// ref-count
		public:
			void AddRef()
			{
				RefCount.fetch_add(1, std::memory_order_relaxed);
			}

			void Release()
			{
				uint32 LocalRefCount = RefCount.fetch_sub(1, std::memory_order_acq_rel) - 1;
				if (LocalRefCount == 0)
				{
#if !defined(__clang_analyzer__)
					delete this;
#endif
				}
			}

			uint32 GetRefCount(std::memory_order MemoryOrder = std::memory_order_relaxed) const
			{
				return RefCount.load(MemoryOrder);
			}

		private:
			std::atomic<uint32> RefCount;
			////////////////////////////////////////////////////////////////////////////

		protected:
			explicit FTaskBase(uint32 InitRefCount, bool bUnlockPrerequisites = true)
				: RefCount(InitRefCount)
			{
				if (bUnlockPrerequisites)
				{
					Prerequisites.Unlock();
				}
			}

			void Init(const TCHAR* InDebugName, ETaskPriority InPriority, EExtendedTaskPriority InExtendedPriority, ETaskFlags Flags)
			{
				// store debug name, priority and an adaptor for task execution in low-level task. The task body can't be stored as this task
				// implementation needs to do some accounting before the task is executed (e.g. maintainance of TLS "current task")
				LowLevelTasks::ETaskFlags LowLevelTaskFlags = LowLevelTasks::ETaskFlags::DefaultFlags;
				LowLevelTask.Init(InDebugName, InPriority,
					[
						this,
						// releasing scheduler's task reference can cause task's automatic destruction and so must be done after the low-level task
						// task is flagged as completed. The task is flagged as completed after the continuation is executed but before its destroyed.
						// `Deleter` is captured by value and is destroyed along with the continuation, calling the given functor on destruction
						Deleter = LowLevelTasks::TDeleter<FTaskBase, &FTaskBase::Release>{ this }
					]
					{
						TryExecuteTask();
					},
					LowLevelTaskFlags
				);
				ExtendedPriority = InExtendedPriority;

				CaptureInheritedContext();
			}

			virtual ~FTaskBase()
			{
				check(IsCompleted());
				TaskTrace::Destroyed(GetTraceId());
			}

			virtual void ExecuteTask() = 0;

		public:
			// returns true if it's valid to wait for the task completion.
			// it's not valid to wait for a task e.g. from inside task's execution, as this would deadlock
			bool IsAwaitable() const
			{
				return FPlatformTLS::GetCurrentThreadId() != ExecutingThreadId.load(std::memory_order_relaxed);
			}

			bool IsNamedThreadTask() const
			{
				return ExtendedPriority >= EExtendedTaskPriority::GameThreadNormalPri;
			}

			ETaskPriority GetPriority() const
			{
				return LowLevelTask.GetPriority();
			}

			EExtendedTaskPriority GetExtendedPriority() const
			{
				return ExtendedPriority;
			}

			// The task will be executed only when all prerequisites are completed. The task type must be a task handle that holds a pointer to
			// FTaskBase as its `Pimpl` member (see Tasks::TTaskBase).
			// Must not be called concurrently
			bool AddPrerequisites(FTaskBase& Prerequisite)
			{
				TASKGRAPH_VERBOSE_EVENT_SCOPE(FTaskBase::AddPrerequisites_Single);

				checkf(NumLocks.load(std::memory_order_relaxed) >= NumInitialLocks && NumLocks.load(std::memory_order_relaxed) < ExecutionFlag, TEXT("Prerequisites can be added only before the task is launched"));

				// registering the task as a subsequent of the given prerequisite can cause its immediate launch by the prerequisite
				// (if the prerequisite has been completed on another thread), so we need to keep the task locked by assuming that the 
				// prerequisite can be added successfully, and release the lock if it wasn't
				uint32 PrevNumLocks = NumLocks.fetch_add(1, std::memory_order_relaxed); // relaxed because the following
				// `AddSubsequent` provides required sync
				checkf(PrevNumLocks + 1 < ExecutionFlag, TEXT("Max number of task prerequisites reached: %d"), ExecutionFlag);

				if (!Prerequisite.AddSubsequent(*this)) // linearisation point, acq_rel semantic
				{
					// failed to add the prerequisite (too late), correct the number
					NumLocks.fetch_sub(1, std::memory_order_relaxed); // relaxed because the previous `AddSubsequent` call provides required sync
					return false;
				}

				Prerequisite.AddRef(); // keep it alive until this task's execution
				Prerequisites.Push(&Prerequisite); // release memory order
				return true;
			}

			// The task will be executed only when all prerequisites are completed. The task type must be a task handle that holds a pointer to
			// FTaskBase as its `Pimpl` member (see Tasks::TTaskBase).
			// Must not be called concurrently
			template<typename HigherLevelTaskType, decltype(std::declval<HigherLevelTaskType>().Pimpl)* = nullptr>
			bool AddPrerequisites(const HigherLevelTaskType& Prerequisite)
			{
				return Prerequisite.IsValid() ? AddPrerequisites(*Prerequisite.Pimpl) : false;
			}

			// The task will be executed only when all prerequisites are completed. The task type must be a task handle that holds a pointer to
			// Must not be called concurrently
			template<typename HigherLevelTaskType, std::enable_if_t<std::is_same_v<HigherLevelTaskType, FGraphEventRef>>* = nullptr>
			bool AddPrerequisites(const HigherLevelTaskType& Prerequisite)
			{
				return Prerequisite.IsValid() ? AddPrerequisites(*Prerequisite.GetReference()) : false;
			}

protected:
			// The task will be executed only when all prerequisites are completed.
			// Must not be called concurrently.
			// @param InPrerequisites - an iterable collection of tasks
			template<typename PrerequisiteCollectionType, decltype(std::declval<PrerequisiteCollectionType>().begin())* = nullptr>
			void AddPrerequisites(const PrerequisiteCollectionType& InPrerequisites, bool bLockPrerequisite)
			{
				const int32 PrerequisiteCount = GetNum(InPrerequisites);
				if (UNLIKELY(PrerequisiteCount <= 0))
				{
					return;
				}

				TASKGRAPH_VERBOSE_EVENT_SCOPE(FTaskBase::AddPrerequisites_Collection);

				checkf(NumLocks.load(std::memory_order_relaxed) >= NumInitialLocks && NumLocks.load(std::memory_order_relaxed) < ExecutionFlag, TEXT("Prerequisites can be added only before the task is launched"));

				// registering the task as a subsequent of the given prerequisite can cause its immediate launch by the prerequisite
				// (if the prerequisite has been completed on another thread), so we need to keep the task locked by assuming that the 
				// prerequisite can be added successfully, and release the lock if it wasn't
				uint32 PrevNumLocks = NumLocks.fetch_add(PrerequisiteCount, std::memory_order_relaxed); // relaxed because the following
				// `AddSubsequent` provides required sync

				bool bUnlockPrerequisites = false;

				uint32 NumCompletedPrerequisites = 0;
				for (auto& Prereq : InPrerequisites)
				{
					// prerequisites can be either `FTaskBase*` or its Pimpl handle
					FTaskBase* Prerequisite;
					using FPrerequisiteType = std::decay_t<decltype(*std::declval<PrerequisiteCollectionType>().begin())>;
					if constexpr (std::is_same_v<FPrerequisiteType, FTaskBase*>)
					{
						Prerequisite = Prereq;
					}
					else if constexpr (std::is_same_v<FPrerequisiteType, FGraphEventRef>)
					{
						Prerequisite = Prereq.GetReference();
					}
					else if constexpr (std::is_pointer_v<FPrerequisiteType>)
					{
						Prerequisite = Prereq->Pimpl;
					}
					else
					{
						Prerequisite = Prereq.Pimpl;
					}

					if (Prerequisite == nullptr)
					{
						++NumCompletedPrerequisites;
						continue;
					}

					if (Prerequisite->AddSubsequent(*this)) // acq_rel memory order
					{
						Prerequisite->AddRef(); // keep it alive until this task's execution
						if (bLockPrerequisite)
						{
							Prerequisites.Lock();
							bLockPrerequisite = false; // don't lock again
							bUnlockPrerequisites = true;
						}

						Prerequisites.PushNoLock(Prerequisite); // relaxed memory order
					}
					else
					{
						++NumCompletedPrerequisites;
					}
				}

				// This check is here to avoid the data dependency on PrevNumLocks.
				checkf(PrevNumLocks + GetNum(InPrerequisites) < ExecutionFlag, TEXT("Max number of nested tasks reached: %d"), ExecutionFlag);

				if (bUnlockPrerequisites)
				{
					Prerequisites.Unlock();
				}

				// unlock for prerequisites that weren't added
				if (NumCompletedPrerequisites)
				{
					NumLocks.fetch_sub(NumCompletedPrerequisites, std::memory_order_release);
				}
			}
public:
			// The task will be executed only when all prerequisites are completed.
			// Must not be called concurrently.
			// @param InPrerequisites - an iterable collection of tasks
			template<typename PrerequisiteCollectionType, decltype(std::declval<PrerequisiteCollectionType>().begin())* = nullptr>
			void AddPrerequisites(const PrerequisiteCollectionType& InPrerequisites)
			{
				AddPrerequisites(InPrerequisites, true /* bLockPrerequisites */);
			}

			// the task unlocks all its subsequents on completion.
			// returns false if the task is already completed and the subsequent wasn't added
			bool AddSubsequent(FTaskBase& Subsequent)
			{
				if (Subsequents.PushIfNotClosed(&Subsequent))
				{
					TaskTrace::SubsequentAdded(GetTraceId(), Subsequent.GetTraceId());
					return true;
				}
				return false;
			}

			// A piped task is executed after the previous task from this pipe is completed. Tasks from the same pipe are not executed
			// concurrently (so don't require synchronization), but not necessarily on the same thread.
			// @See FPipe
			void SetPipe(FPipe& InPipe)
			{
				// keep the task locked until it's pushed into the pipe
				NumLocks.fetch_add(1, std::memory_order_relaxed); // the order doesn't matter as this happens before the task is launched
				Pipe = &InPipe;
			}

			FPipe* GetPipe() const
			{
				return Pipe;
			}

			// Tries to schedule task execution but has additional tracking to ensure a task is not scheduled more than once. 
			// This method should only be used when scheduling more than once is allowed/practical (e.g. FTaskEvent).
			// Returns false if the task has been triggered before or has incomplete dependencies (prerequisites or is blocked by a pipe). 
			// In the latter case, the task will be automatically scheduled when all dependencies are completed.
			bool Trigger(uint64 TaskSize)
			{
				if (TaskTriggered.exchange(true, std::memory_order_relaxed))
				{
					// Task is already launched
					return false;
				}

				// Now that we know we have triggered the task (it might not yet be launched/running based on dependencies) it's now in the
				// task system and can outlive external references, so we need to keep it alive by holding an internal reference. 
				// It will be released when completed.
				AddRef();
				return TryLaunch(TaskSize);
			}

			// Tries to schedule task execution. Returns false if the task has incomplete dependencies (prerequisites or is blocked by a pipe). 
			// In this case the task will be automatically scheduled when all dependencies are completed.
			bool TryLaunch(uint64 TaskSize)
			{
#if UE_TASK_TRACE_ENABLED
				if (UE_TRACE_CHANNELEXPR_IS_ENABLED(TaskTrace::TaskChannel))
				{
					// The params to this are slow to compute
					TaskTrace::Launched(GetTraceId(), LowLevelTask.GetDebugName(), true, TranslatePriority(LowLevelTask.GetPriority(), ExtendedPriority), TaskSize);
				}
#endif

				bool bWakeUpWorker = true;
				return TryUnlock(bWakeUpWorker);
			}

			// @return true if the task was executed and all its nested tasks are completed
			bool IsCompleted() const
			{
				return Subsequents.IsClosed();
			}

			// Tries to pull out the task from the system and execute it. If the task is locked by either prerequisites or nested tasks, tries to 
			// retract and execute them recursively. 
			// WARNING: the function can return `true` even if the task is not completed yet. The `true` means only that the task is already
			// executed and has no other pending dependencies, but can be in the process of completion (concurrently). The caller still needs 
			// to wait for completion explicitly.
			CORE_API bool TryRetractAndExecute(FTimeout Timeout, uint32 RecursionDepth = 0);

			// releases internal reference and maintains low-level task state. must be called iff the task was never launched, otherwise 
			// the scheduler will do this in due course
			void ReleaseInternalReference()
			{
				verify(LowLevelTask.TryCancel());
			}

			// adds a nested task that must be completed before the parent (this) is completed
			void AddNested(FTaskBase& Nested)
			{
				TASKGRAPH_VERBOSE_EVENT_SCOPE(FTaskBase::AddNested);

				uint32 PrevNumLocks = NumLocks.fetch_add(1, std::memory_order_relaxed); // in case we'll succeed in adding subsequent, 
				// "happens before" registering this task as a subsequent
				checkf(PrevNumLocks + 1 < TNumericLimits<uint32>::Max(), TEXT("Max number of nested tasks reached: %d"), TNumericLimits<uint32>::Max() - ExecutionFlag);
				checkf(PrevNumLocks > ExecutionFlag, TEXT("Internal error: nested tasks can be added only during parent's execution (%u)"), PrevNumLocks);

				if (Nested.AddSubsequent(*this)) // "release" memory order
				{
					Nested.AddRef(); // keep it alive as we store it in `Prerequisites` and we can need it to try to retract it. it's released on closing the task
					Prerequisites.Push(&Nested);
				}
				else
				{
					NumLocks.fetch_sub(1, std::memory_order_relaxed);
				}
			}

			// waits for task's completion, with optional timeout. Tries to retract the task and execute it in-place, if failed - blocks until the task 
			// is completed by another thread. If timeout is zero, tries to retract the task and returns immedially after that. 
			// `Wait(FTimespan::Zero())` still tries to retract and execute the task, use `IsCompleted()` to check for completeness. 
			// The version w/o timeout is slightly more efficient.
			// @return true if the task is completed
			CORE_API bool Wait(FTimeout Timeout);

			// waits for task's completion. Tries to retract the task and execute it in-place, if failed - blocks until the task 
			// is completed by another thread. 
			CORE_API void Wait();

			// mimics the old tasks (TaskGraph) behaviour on named threads: waiting for a task on a named thread pulls other tasks from this
			// named thread queue and executes them
			CORE_API void WaitWithNamedThreadsSupport();

			TaskTrace::FId GetTraceId() const
			{
#if UE_TASK_TRACE_ENABLED
				return TraceId.load(std::memory_order_relaxed);
#else
				return TaskTrace::InvalidId;
#endif
			}

		protected:
			// tries to get execution permission and if successful, executes given task body and completes the task if there're no pending nested tasks. 
			// does all required accounting before/after task execution. the task can be deleted as a result of this call.
			// @returns true if the task was executed by the current thread
			bool TryExecuteTask()
			{
				TASKGRAPH_VERBOSE_EVENT_SCOPE(FTaskBase::TryExecuteTask);

				if (!TrySetExecutionFlag())
				{
					return false;
				}

				AddRef(); // `LowLevelTask` will automatically release the internal reference after execution, but there can be pending nested tasks, so keep it alive
				// it's released either later here if the task is closed, or when the last nested task is completed and unlocks its parent (in `TryUnlock`)

				ReleasePrerequisites();

				FTaskBase* PrevTask = ExchangeCurrentTask(this);
				ExecutingThreadId.store(FPlatformTLS::GetCurrentThreadId(), std::memory_order_relaxed);

				if (GetPipe() != nullptr)
				{
					StartPipeExecution();
				}

				{
					UE::FInheritedContextScope InheritedContextScope = RestoreInheritedContext();
					TaskTrace::FTaskTimingEventScope TaskEventScope(GetTraceId());
					TASKGRAPH_VERBOSE_EVENT_SCOPE(FTaskBase::ExecuteTask);
					ExecuteTask();
				}

				if (GetPipe() != nullptr)
				{
					FinishPipeExecution();
				}

				ExecutingThreadId.store(FThread::InvalidThreadId, std::memory_order_relaxed); // no need to sync with loads as they matter only if
				// executed by the same thread
				ExchangeCurrentTask(PrevTask);

				// close the task if there are no pending nested tasks
				uint32 LocalNumLocks = NumLocks.fetch_sub(1, std::memory_order_acq_rel) - 1; // "release" to make task execution "happen before" this, and "acquire" to 
				// "sync with" another thread that completed the last nested task
				if (LocalNumLocks == ExecutionFlag) // unlocked (no pending nested tasks)
				{
					Close();
					Release(); // the internal reference that kept the task alive for nested tasks
				} // else there're non completed nested tasks, the last one will unlock, close and release the parent (this task)

				return true;
			}

			// closes task by unlocking its subsequents and flagging it as completed
			CORE_API void Close();

			CORE_API void ClearPipe();

		private:
			// A task can be locked for execution (by prerequisites or if it's not launched yet) or for completion (by nested tasks).
			// This method is called to unlock the task and so can result in its scheduling (and execution) or completion
			bool TryUnlock(bool& bWakeUpWorker)
			{
				TASKGRAPH_VERBOSE_EVENT_SCOPE(FTaskBase::TryUnlock);

				FPipe* LocalPipe = GetPipe(); // cache data locally so we won't need to touch the member (read below)

				uint32 PrevNumLocks = NumLocks.fetch_sub(1, std::memory_order_acq_rel); // `acq_rel` to make it happen after task 
				// preparation and before launching it
				// the task can be dead already as the prev line can remove the lock hold for this execution path, another thread(s) can unlock
				// the task, execute, complete and delete it. thus before touching any members or calling methods we need to make sure
				// the task can't be destroyed concurrently

				uint32 LocalNumLocks = PrevNumLocks - 1;

				if (PrevNumLocks < ExecutionFlag)
				{
					// pre-execution state, try to schedule the task

					checkf(PrevNumLocks != 0, TEXT("The task is not locked"));

					bool bPrerequisitesCompleted = LocalPipe == nullptr ? LocalNumLocks == 0 : LocalNumLocks <= 1; // the only remaining lock is pipe's one (if any)
					if (!bPrerequisitesCompleted)
					{
						return false;
					}

					// this thread unlocked the task, no other thread can reach this point concurrently, we can touch the task again

					if (LocalPipe != nullptr)
					{
						bool bFirstPipingAttempt = LocalNumLocks == 1;
						if (bFirstPipingAttempt)
						{
							FTaskBase* PrevPipedTask = TryPushIntoPipe();
							if (PrevPipedTask != nullptr) // the pipe is blocked
							{
								// the prev task in pipe's chain becomes this task's prerequisite, to enabled piped task retraction.
								// its ref count already accounted for this ref. the ref will be released when the prereq is not needed anymore
								Prerequisites.Push(PrevPipedTask);
								return false;
							}

							NumLocks.store(0, std::memory_order_release); // release pipe's lock
						}
					}

					if (ExtendedPriority == EExtendedTaskPriority::Inline)
					{
						// "inline" tasks are not scheduled but executed straight away
						TryExecuteTask(); // result doesn't matter, this can fail if task retraction jumped in and got execution
						// permission between this thread unlocked the task and tried to execute it
						ReleaseInternalReference();

						// Use-after-free territory, do not touch any of the task's properties here.
					}
					else if (ExtendedPriority == EExtendedTaskPriority::TaskEvent)
					{
						// task events have nothing to execute, try to close it. task retraction can jump in and close the task event, 
						// so this thread still needs to check execution permission
						if (TrySetExecutionFlag())
						{
							// task events are used as an empty prerequisites/subsequents
							ReleasePrerequisites();
							Close();
							ReleaseInternalReference();

							// Use-after-free territory, do not touch any of the task's properties here.
						}
					}
					else
					{
						Schedule(bWakeUpWorker);

						// Use-after-free territory, do not touch any of the task's properties here.
					}

					return true;
				}

				// execution already started (at least), this is nested tasks unlocking their parent
				checkf(PrevNumLocks != ExecutionFlag, TEXT("The task is not locked"));
				if (LocalNumLocks != ExecutionFlag) // still locked
				{
					return false;
				}

				// this thread unlocked the task, no other thread can reach this point concurrently, we can touch the task again
				Close();
				Release(); // the internal reference that kept the task alive for nested tasks

				// Use-after-free territory, do not touch any of the task's properties here.

				return true;
			}

			CORE_API void Schedule(bool& bWakeUpWorker);

			// is called when the task has no pending prerequisites. Returns the previous piped task if any
			CORE_API FTaskBase* TryPushIntoPipe();

			// only one thread can successfully set execution flag, that grants task execution permission
			// @returns false if another thread got execution permission first
			bool TrySetExecutionFlag()
			{
				uint32 ExpectedUnlocked = 0;
				// set the execution flag and simultenously lock it (+1) so a nested task completion doesn't close it before its execution is finished
				return NumLocks.compare_exchange_strong(ExpectedUnlocked, ExecutionFlag + 1, std::memory_order_acq_rel, std::memory_order_relaxed); // on success 
				// - linearisation point for task execution, on failure - load order doesn't matter
			}

			void ReleasePrerequisites()
			{
				TASKGRAPH_VERBOSE_EVENT_SCOPE(FTaskBase::ReleasePrerequisites);
				for (FTaskBase* Prerequisite : Prerequisites.PopAll())
				{
					TASKGRAPH_VERBOSE_EVENT_SCOPE(FTaskBase::ReleasePrerequisite);
					Prerequisite->Release();
				}
			}

			CORE_API void StartPipeExecution();
			CORE_API void FinishPipeExecution();

			CORE_API bool WaitImpl(FTimeout Timeout);

		private:
			// the number of times that the task should be unlocked before it can be scheduled or completed
			// initial count is 1 for launching the task (it can't be scheduled before it's launched)
			// reaches 0 the task is scheduled for execution.
			// NumLocks's the most significant bit (see `ExecutionFlag`) is set on task execution start, and indicates that now 
			// NumLocks is about how many times the task must be unlocked to be completed
			static constexpr uint32 NumInitialLocks = 1;
			std::atomic<uint32> NumLocks{ NumInitialLocks };

			FPipe* Pipe{ nullptr };

			FEventCount StateChangeEvent;

			EExtendedTaskPriority ExtendedPriority; // internal priorities, if any

			// Note: Only ever set when the task is launched from Trigger()
			std::atomic<bool> TaskTriggered = false;

			std::atomic<uint32> ExecutingThreadId = FThread::InvalidThreadId;

#if UE_TASK_TRACE_ENABLED
			std::atomic<TaskTrace::FId> TraceId{ TaskTrace::GenerateTaskId() };
#endif

			// stores backlinks to prerequsites, either execution prerequisites or nested tasks (completion prerequisites).
			// It's populated in three stages:
			// 1) by adding execution prerequisites, before the task is launched.
			// 2) by piping, when the previous piped task (if any) is added as a prerequisite. can happen concurrently with other threads accessing prerequisites for
			//		task retraction.
			// 3) by adding nested tasks. after piping. during task execution.
			template <typename AllocatorType = FDefaultAllocator>
			class FPrerequisites
			{
			public:
				void Push(FTaskBase* Prerequisite)
				{
					TASKGRAPH_VERBOSE_EVENT_SCOPE(FPrerequisites::Push);
					UE::TUniqueLock Lock(Mutex);
					Prerequisites.Emplace(Prerequisite);
				}

				void PushNoLock(FTaskBase* Prerequisite)
				{
					TASKGRAPH_VERBOSE_EVENT_SCOPE(FPrerequisites::PushNoLock);
					Prerequisites.Emplace(Prerequisite);
				}

				TArray<FTaskBase*, AllocatorType> PopAll()
				{
					TASKGRAPH_VERBOSE_EVENT_SCOPE(FPrerequisites::PopAll);
					UE::TUniqueLock Lock(Mutex);
					return MoveTemp(Prerequisites);
				}

				void Lock()
				{
					Mutex.Lock();
				}

				void Unlock()
				{
					Mutex.Unlock();
				}
			private:
				TArray<FTaskBase*, AllocatorType> Prerequisites;
				UE::FMutex Mutex{ UE::AcquireLock }; // Start locked by default to avoid compare exchange during construction.
			};

			FPrerequisites<TInlineAllocator<1>> Prerequisites;

			LowLevelTasks::FTask LowLevelTask;

			// the task is completed when its subsequents list is closed and no more can be added
			template <typename AllocatorType = FDefaultAllocator>
			class FSubsequents
			{
			public:
				bool PushIfNotClosed(FTaskBase* NewItem)
				{
					TASKGRAPH_VERBOSE_EVENT_SCOPE(FSubsequents::PushIfNotClosed);
					// AddSubsequent expects acquire as some code can use the result here
					// to decide if a task is finished and if the memory it produced/modified
					// is safe to access.
					if (bIsClosed.load(std::memory_order_acquire))
					{
						return false;
					}
					UE::TUniqueLock Lock(Mutex);
					if (bIsClosed)
					{
						return false;
					}
					Subsequents.Emplace(NewItem);
					return true;
				}

				TArray<FTaskBase*, AllocatorType> Close()
				{
					TASKGRAPH_VERBOSE_EVENT_SCOPE(FSubsequents::Close);
					UE::TUniqueLock Lock(Mutex);
					bIsClosed = true;
					return MoveTemp(Subsequents);
				}

				bool IsClosed() const
				{
					return bIsClosed;
				}

			private:
				TArray<FTaskBase*, AllocatorType> Subsequents;
				std::atomic<bool>  bIsClosed = false;
				UE::FMutex Mutex;
			};

			FSubsequents<TInlineAllocator<1>> Subsequents;


protected:
			void UnlockPrerequisites()
			{
				Prerequisites.Unlock();
			}
		};

		// an extension of FTaskBase for tasks that return a result.
		// Stores task execution result and provides an access to it.
		template<typename ResultType>
		class TTaskWithResult : public FTaskBase
		{
		protected:
			explicit TTaskWithResult(const TCHAR* InDebugName, ETaskPriority InPriority, EExtendedTaskPriority InExtendedPriority, uint32 InitRefCount, ETaskFlags Flags)
				: FTaskBase(InitRefCount)
			{
				Init(InDebugName, InPriority, InExtendedPriority, Flags);
			}

			virtual ~TTaskWithResult() override
			{
				DestructItem(ResultStorage.GetTypedPtr());
			}

		public:
			ResultType& GetResult()
			{
				checkf(IsCompleted(), TEXT("The task must be completed to obtain its result"));
				return *ResultStorage.GetTypedPtr();
			}

		protected:
			TTypeCompatibleBytes<ResultType> ResultStorage;
		};

		// Task implementation that can be executed, as it stores task body. Generic version (for tasks that return non-void results).
		// In most cases it should be allocated on the heap and used with TRefCountPtr, e.g. @see FTaskHandle. 
		template<typename TaskBodyType, typename ResultType = TInvokeResult_T<TaskBodyType>, typename Enable = void>
		class TExecutableTaskBase : public TTaskWithResult<ResultType>
		{
			UE_NONCOPYABLE(TExecutableTaskBase);

		public:
			virtual void ExecuteTask() override final
			{
				new(&this->ResultStorage) ResultType{ Invoke(*TaskBodyStorage.GetTypedPtr()) };

				// destroy the task body as soon as we are done with it, as it can have captured data sensitive to destruction order
				DestructItem(TaskBodyStorage.GetTypedPtr());
			}

		protected:
			TExecutableTaskBase(const TCHAR* InDebugName, TaskBodyType&& TaskBody, ETaskPriority InPriority, EExtendedTaskPriority InExtendedPriority, ETaskFlags Flags)
				: TTaskWithResult<ResultType>(InDebugName, InPriority, InExtendedPriority, 2, Flags)
				// 2 init refs: one for the initial reference (we don't increment it on passing to `TRefCountPtr`), and one for the internal 
				// reference that keeps the task alive while it's in the system. is released either on task completion or by the scheduler after
				// trying to execute the task
			{
				new(&TaskBodyStorage) TaskBodyType(MoveTemp(TaskBody));
			}

		private:
			TTypeCompatibleBytes<TaskBodyType> TaskBodyStorage;
		};

		// a specialization for tasks that don't return results
		template<typename TaskBodyType>
		class TExecutableTaskBase<TaskBodyType, typename TEnableIf<std::is_same_v<TInvokeResult_T<TaskBodyType>, void>>::Type> : public FTaskBase
		{
			UE_NONCOPYABLE(TExecutableTaskBase);

		public:
			virtual void ExecuteTask() override final
			{
				Invoke(*TaskBodyStorage.GetTypedPtr());

				// destroy the task body as soon as we are done with it, as it can have captured data sensitive to destruction order
				DestructItem(TaskBodyStorage.GetTypedPtr());
			}

		protected:
			TExecutableTaskBase(const TCHAR* InDebugName, TaskBodyType&& TaskBody, ETaskPriority InPriority, EExtendedTaskPriority InExtendedPriority, ETaskFlags Flags) :
				FTaskBase(2) // 2 init refs: one for the initial reference (we don't increment it on passing to `TRefCountPtr`), and one for the internal 
				// reference that keeps the task alive while it's in the system. is released either on task completion or by the scheduler after
				// trying to execute the task
			{
				Init(InDebugName, InPriority, InExtendedPriority, Flags);
				new(&TaskBodyStorage) TaskBodyType(MoveTemp(TaskBody));
			}

		private:
			TTypeCompatibleBytes<TaskBodyType> TaskBodyStorage;
		};

		inline constexpr int32 SmallTaskSize = 256;
		using FExecutableTaskAllocator = TLockFreeFixedSizeAllocator_TLSCache<SmallTaskSize, PLATFORM_CACHE_LINE_SIZE>;
		CORE_API extern FExecutableTaskAllocator SmallTaskAllocator;

		// a separate derived class to add "small task" allocation optimization to both base class specializations
		template<typename TaskBodyType>
		class alignas(PLATFORM_CACHE_LINE_SIZE) TExecutableTask final : public TExecutableTaskBase<TaskBodyType>
		{
		public:
			TExecutableTask(const TCHAR* InDebugName, TaskBodyType&& TaskBody, ETaskPriority InPriority, EExtendedTaskPriority InExtendedPriority, ETaskFlags Flags)
				: TExecutableTaskBase<TaskBodyType>(InDebugName, MoveTemp(TaskBody), InPriority, InExtendedPriority, Flags)
			{
			}

			// a helper that deduces the template argument
			static TExecutableTask* Create(const TCHAR* InDebugName, TaskBodyType&& TaskBody, ETaskPriority InPriority, EExtendedTaskPriority InExtendedPriority, ETaskFlags Flags)
			{
				return new TExecutableTask(InDebugName, MoveTemp(TaskBody), InPriority, InExtendedPriority, Flags);
			}

			static void* operator new(size_t Size)
			{
				if (Size <= SmallTaskSize)
				{
					 return SmallTaskAllocator.Allocate();
				}
				else
				{
					TASKGRAPH_VERBOSE_EVENT_SCOPE(TExecutableTask::LargeAlloc);
					return FMemory::Malloc(sizeof(TExecutableTask), alignof(TExecutableTask));
				}
			}

			static void operator delete(void* Ptr, size_t Size)
			{
				Size <= SmallTaskSize ? SmallTaskAllocator.Free(Ptr) : FMemory::Free(Ptr);
			}
		};

		// waiting on named threads that replicates TaskGraph logic
		// returns true if called on a named thread
		CORE_API bool TryWaitOnNamedThread(FTaskBase& Task);

		// a special kind of task that is used for signalling or dependency management. It can have prerequisites or be used as a prerequisite for other tasks. 
		// It's optimized for the fact that it doesn't have a task body and so doesn't need to be scheduled and executed
		class FTaskEventBase : public FTaskBase
		{
		public:
			static FTaskEventBase* Create(const TCHAR* DebugName)
			{
				return new FTaskEventBase(DebugName);
			}

			static void* operator new(size_t Size);
			static void operator delete(void* Ptr);

		private:
			FTaskEventBase(const TCHAR* InDebugName)
				: FTaskBase(/*InitRefCount=*/ 1) // for the initial reference (we don't increment it on passing to `TRefCountPtr`)
			{
				TaskTrace::Created(GetTraceId(), sizeof(*this));
				Init(InDebugName, ETaskPriority::Normal, EExtendedTaskPriority::TaskEvent, ETaskFlags::None);
			}

			virtual void ExecuteTask() override final
			{
				checkNoEntry(); // never executed because it doesn't have a task body
			}
		};

		using FTaskEventBaseAllocator = TLockFreeFixedSizeAllocator_TLSCache<sizeof(FTaskEventBase), PLATFORM_CACHE_LINE_SIZE>;
		CORE_API extern FTaskEventBaseAllocator TaskEventBaseAllocator;

		inline void* FTaskEventBase::operator new(size_t Size)
		{
			return TaskEventBaseAllocator.Allocate();
		}

		inline void FTaskEventBase::operator delete(void* Ptr)
		{
			TaskEventBaseAllocator.Free(Ptr);
		}

		// task retraction of multiple tasks, with timeout. The timeout is rounded up to any successful task execution, which means that it can 
		// time out only in-between individual task retractions.
		// WARNING: the function can return `true` even if some tasks are still not completed. The `true` means only that the tasks are executed
		// and have no other pending dependencies, but can be still in the process of completion (concurrently). The caller still needs to wait 
		// for completion.
		template<typename TaskCollectionType>
		bool TryRetractAndExecute(const TaskCollectionType& Tasks, FTimeout Timeout)
		{
			bool bResult = true;

			for (auto& Task : Tasks)
			{
				if (Task.IsValid() && !Task.Pimpl->TryRetractAndExecute(Timeout))
				{
					bResult = false;  // do not stop here to let this thread to help in executing tasks as much as possible, as it's waiting for their completion anyway
				}

				if (Timeout.IsExpired())
				{
					return false;
				}
			}

			return bResult;
		}
		
		template<typename TaskCollectionType>
		TArray<TaskTrace::FId> GetTraceIds(const TaskCollectionType& Tasks)
		{
#if UE_TASK_TRACE_ENABLED
			TArray<TaskTrace::FId> TasksIds;
			TasksIds.Reserve(Tasks.Num());

			for (auto& Task : Tasks)
			{
				if (Task.IsValid())
				{
					TasksIds.Add(Task.Pimpl->GetTraceId());
				}
			}

			return TasksIds;
#else
			return {};
#endif
		}
	}
}
