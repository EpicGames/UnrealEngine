// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Concepts/Integral.h"
#include "Concepts/Pointer.h"
#include "HAL/PlatformMisc.h"
#include <type_traits>

/**
 * Aligns a value to the nearest higher multiple of 'Alignment', which must be a power of two.
 *
 * @param  Val        The value to align.
 * @param  Alignment  The alignment value, must be a power of two.
 *
 * @return The value aligned up to the specified alignment.
 */
template <typename T>
	requires UE::CIntegral<T> || UE::CPointer<T>
UE_FORCEINLINE_HINT constexpr T Align(T Val, uint64 Alignment)
{
	return (T)(((uint64)Val + Alignment - 1) & ~(Alignment - 1));
}

/**
 * Aligns a value to the nearest lower multiple of 'Alignment', which must be a power of two.
 *
 * @param  Val        The value to align.
 * @param  Alignment  The alignment value, must be a power of two.
 *
 * @return The value aligned down to the specified alignment.
 */
template <typename T>
	requires UE::CIntegral<T> || UE::CPointer<T>
UE_FORCEINLINE_HINT constexpr T AlignDown(T Val, uint64 Alignment)
{
	return (T)(((uint64)Val) & ~(Alignment - 1));
}

/**
 * Checks if a pointer is aligned to the specified alignment.
 *
 * @param  Val        The value to align.
 * @param  Alignment  The alignment value, must be a power of two.
 *
 * @return true if the pointer is aligned to the specified alignment, false otherwise.
 */
template <typename T>
	requires UE::CIntegral<T> || UE::CPointer<T>
UE_FORCEINLINE_HINT constexpr bool IsAligned(T Val, uint64 Alignment)
{
	return !((uint64)Val & (Alignment - 1));
}

/**
 * Aligns a value to the nearest higher multiple of 'Alignment'.
 *
 * @param  Val        The value to align.
 * @param  Alignment  The alignment value, can be any arbitrary value.
 *
 * @return The value aligned up to the specified alignment.
 */
template <typename T>
	requires UE::CIntegral<T> || UE::CPointer<T>
UE_FORCEINLINE_HINT constexpr T AlignArbitrary(T Val, uint64 Alignment)
{
	return (T)((((uint64)Val + Alignment - 1) / Alignment) * Alignment);
}

/**
 * Gets the size of the properties of a type, without any trailing padding. Used to populate PropertiesSize for UStructs for native types.
 * This is important because we use the offset of properties to determine what class they are in (i.e. an offset that falls within a parent
 * class is assumed to be a property of the parent class). Since compilers can reuse the tail padding of a class in derived classes, PropertiesSize
 * must be no more than the offset of the first byte of a derived class for non-final classes.
 * Unfortunately, this implementation does not work for final types, so we fall back to just using sizeof(T) which what was used previously
 * Note that for types that have a virtual destructor, we declare a pure virtual destructor in Derived in order to avoid calling the destructor
 *     in T and triggering compile errors related to private destructors or destructors for incomplete types
 *
 * @return sizeof(T) minus the unused tail alignment bytes that subclasses may reuse. Unaligned.
 */

template <typename T>
consteval std::size_t DataSizeOf()
{
	if constexpr (std::is_final_v<T>)
	{
		// Final type. We can't check the offset of a child property so we just use sizeof
		return sizeof(T);
	}
	else if constexpr(std::is_polymorphic_v<T>)
	{
		// Polymorhpic type. Use a pure virtual destructor to avoid compiler errors related to accessing private destructors
#if defined(_MSC_VER) && (_MSC_VER < 1950)
		// There was a bug in msvc that incorrectly reported "= 0" as not meaning "pure virtual" in structs defined in templates
		__pragma(warning(push)) __pragma(warning(disable : 5288))
#endif
			struct PLATFORM_EMPTY_BASES Derived : T { virtual ~Derived() = 0; char Byte; };
#if defined(_MSC_VER) && (_MSC_VER < 1950)
		__pragma(warning(pop))
#endif
		constexpr std::size_t DataSize = offsetof(Derived, Byte);
		static_assert(DataSize <= sizeof(T));
		return (DataSize < 1) ? std::size_t(1) : DataSize;
	}
	else
	{
		// Non-polymorhpic type. Delete the destructor to avoid compiler errors related to accessing private destructors
		struct PLATFORM_EMPTY_BASES Derived : T { ~Derived() = delete; char Byte; };
		constexpr std::size_t DataSize = offsetof(Derived, Byte);
		static_assert(DataSize <= sizeof(T));
		return (DataSize < 1) ? std::size_t(1) : DataSize;
	}
}

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Templates/IsIntegral.h"
#include "Templates/IsPointer.h"
#endif
