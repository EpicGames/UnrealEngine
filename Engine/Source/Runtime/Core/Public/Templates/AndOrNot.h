// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Does a boolean AND of the ::Value static members of each type, but short-circuits if any Type::Value == false.
 */
template <typename... Types>
struct TAnd;

template <bool LHSValue, typename... RHS>
struct TAndValue
{
	static constexpr bool Value = ::TAnd<RHS...>::value;
	static constexpr bool value = ::TAnd<RHS...>::value;
};

template <typename... RHS>
struct TAndValue<false, RHS...>
{
	static constexpr bool Value = false;
	static constexpr bool value = false;
};

template <typename LHS, typename... RHS>
struct TAnd<LHS, RHS...> : ::TAndValue<LHS::Value, RHS...>
{
};

template <>
struct TAnd<>
{
	static constexpr bool Value = true;
	static constexpr bool value = true;
};

/**
 * Does a boolean OR of the ::Value static members of each type, but short-circuits if any Type::Value == true.
 */
template <typename... Types>
struct TOr;

template <bool LHSValue, typename... RHS>
struct TOrValue
{
	static constexpr bool Value = ::TOr<RHS...>::value;
	static constexpr bool value = ::TOr<RHS...>::value;
};

template <typename... RHS>
struct TOrValue<true, RHS...>
{
	static constexpr bool Value = true;
	static constexpr bool value = true;
};

template <typename LHS, typename... RHS>
struct TOr<LHS, RHS...> : ::TOrValue<LHS::Value, RHS...>
{
};

template <>
struct TOr<>
{
	static constexpr bool Value = false;
	static constexpr bool value = false;
};

/**
 * Does a boolean NOT of the ::Value static members of the type.
 */
template <typename Type>
struct TNot
{
	static constexpr bool Value = !Type::Value;
	static constexpr bool value = !Type::Value;
};

