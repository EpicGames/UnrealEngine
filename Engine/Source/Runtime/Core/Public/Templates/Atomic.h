// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/ThreadSafeCounter.h"
#include "HAL/ThreadSafeCounter64.h"
#include "Templates/IsIntegral.h"
#include "Templates/IsTrivial.h"
#include "Traits/IntType.h"
#include <atomic>

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// `TAtomic` is planned for deprecation. Please use `std::atomic`
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename T>
struct TAtomicBase_Basic;

template <typename ElementType>
struct TAtomicBase_Mutex;

template <typename ElementType, typename DiffType>
struct TAtomicBase_Arithmetic;

template <typename ElementType>
struct TAtomicBase_Pointer;

template <typename ElementType>
struct TAtomicBase_Integral;

enum class EMemoryOrder
{
	// Provides no guarantees that the operation will be ordered relative to any other operation.
	Relaxed,

	// Establishes a single total order of all other atomic operations marked with this.
	SequentiallyConsistent,

	Count
};

namespace UE::Core::Private::Atomic
{
	template <int Size>
	struct TIsSupportedSize
	{
		enum { Value = (Size == 1) || (Size == 2) || (Size == 4) || (Size == 8) };
	};

	template <typename T>
	using TUnderlyingIntegerType_T = TSignedIntType_T<sizeof(T)>;

	template <typename T> struct TIsVoidPointer                       { static constexpr bool Value = false; };
	template <>           struct TIsVoidPointer<               void*> { static constexpr bool Value = true; };
	template <>           struct TIsVoidPointer<const          void*> { static constexpr bool Value = true; };
	template <>           struct TIsVoidPointer<      volatile void*> { static constexpr bool Value = true; };
	template <>           struct TIsVoidPointer<const volatile void*> { static constexpr bool Value = true; };

	template <typename T>
	FORCEINLINE T LoadRelaxed(const volatile T* Element)
	{
		#ifdef __clang__
			TUnderlyingIntegerType_T<T> Result;
			__atomic_load((volatile TUnderlyingIntegerType_T<T>*)Element, &Result, __ATOMIC_RELAXED);
		#else
			auto Result = *(volatile TUnderlyingIntegerType_T<T>*)Element;
		#endif
		return *(const T*)&Result;
	}

	template <typename T>
	FORCEINLINE T Load(const volatile T* Element)
	{
		auto Result = FPlatformAtomics::AtomicRead((volatile TUnderlyingIntegerType_T<T>*)Element);
		return *(const T*)&Result;
	}

	template <typename T>
	FORCEINLINE void StoreRelaxed(const volatile T* Element, T Value)
	{
		#ifdef __clang__
			__atomic_store((volatile TUnderlyingIntegerType_T<T>*)Element, (TUnderlyingIntegerType_T<T>*)&Value, __ATOMIC_RELAXED);
		#else
			*(volatile TUnderlyingIntegerType_T<T>*)Element = *(TUnderlyingIntegerType_T<T>*)&Value;
		#endif
	}

	template <typename T>
	FORCEINLINE void Store(const volatile T* Element, T Value)
	{
		FPlatformAtomics::InterlockedExchange((volatile TUnderlyingIntegerType_T<T>*)Element, *(const TUnderlyingIntegerType_T<T>*)&Value);
	}

	template <typename T>
	FORCEINLINE T Exchange(volatile T* Element, T Value)
	{
		auto Result = FPlatformAtomics::InterlockedExchange((volatile TUnderlyingIntegerType_T<T>*)Element, *(const TUnderlyingIntegerType_T<T>*)&Value);
		return *(const T*)&Result;
	}

	template <typename T>
	FORCEINLINE T IncrementExchange(volatile T* Element)
	{
		auto Result = FPlatformAtomics::InterlockedIncrement((volatile TUnderlyingIntegerType_T<T>*)Element) - 1;
		return *(const T*)&Result;
	}

	template <typename T>
	FORCEINLINE T* IncrementExchange(T* volatile* Element)
	{
		auto Result = FPlatformAtomics::InterlockedAdd((volatile TUnderlyingIntegerType_T<T*>*)Element, sizeof(T));
		return *(T* const*)&Result;
	}

	template <typename T, typename DiffType>
	FORCEINLINE T AddExchange(volatile T* Element, DiffType Diff)
	{
		auto Result = FPlatformAtomics::InterlockedAdd((volatile TUnderlyingIntegerType_T<T>*)Element, (TUnderlyingIntegerType_T<T>)Diff);
		return *(const T*)&Result;
	}

	template <typename T, typename DiffType>
	FORCEINLINE T* AddExchange(T* volatile* Element, DiffType Diff)
	{
		auto Result = FPlatformAtomics::InterlockedAdd((volatile TUnderlyingIntegerType_T<T*>*)Element, (TUnderlyingIntegerType_T<T*>)(Diff * sizeof(T)));
		return *(T* const*)&Result;
	}

	template <typename T>
	FORCEINLINE T DecrementExchange(volatile T* Element)
	{
		auto Result = FPlatformAtomics::InterlockedDecrement((volatile TUnderlyingIntegerType_T<T>*)Element) + 1;
		return *(const T*)&Result;
	}

	template <typename T>
	FORCEINLINE T* DecrementExchange(T* volatile* Element)
	{
		auto Result = FPlatformAtomics::InterlockedAdd((volatile TUnderlyingIntegerType_T<T*>*)Element, -(TUnderlyingIntegerType_T<T*>)sizeof(T));
		return *(T* const*)&Result;
	}

	template <typename T, typename DiffType>
	FORCEINLINE T SubExchange(volatile T* Element, DiffType Diff)
	{
		auto Result = FPlatformAtomics::InterlockedAdd((volatile TUnderlyingIntegerType_T<T>*)Element, -(TUnderlyingIntegerType_T<T>)Diff);
		return *(const T*)&Result;
	}

	template <typename T, typename DiffType>
	FORCEINLINE T* SubExchange(T* volatile* Element, DiffType Diff)
	{
		auto Result = FPlatformAtomics::InterlockedAdd((volatile TUnderlyingIntegerType_T<T*>*)Element, -(TUnderlyingIntegerType_T<T*>)(Diff * sizeof(T)));
		return *(T* const*)&Result;
	}

	template <typename T>
	FORCEINLINE T CompareExchange(volatile T* Element, T ExpectedValue, T NewValue)
	{
		auto Result = FPlatformAtomics::InterlockedCompareExchange((volatile TUnderlyingIntegerType_T<T>*)Element, *(const TUnderlyingIntegerType_T<T>*)&NewValue, *(const TUnderlyingIntegerType_T<T>*)&ExpectedValue);
		return *(const T*)&Result;
	}

	template <typename T>
	FORCEINLINE T AndExchange(volatile T* Element, T AndValue)
	{
		auto Result = FPlatformAtomics::InterlockedAnd((volatile TUnderlyingIntegerType_T<T>*)Element, (TUnderlyingIntegerType_T<T>)AndValue);
		return *(const T*)&Result;
	}

	template <typename T>
	FORCEINLINE T OrExchange(volatile T* Element, T OrValue)
	{
		auto Result = FPlatformAtomics::InterlockedOr((volatile TUnderlyingIntegerType_T<T>*)Element, (TUnderlyingIntegerType_T<T>)OrValue);
		return *(const T*)&Result;
	}

	template <typename T>
	FORCEINLINE T XorExchange(volatile T* Element, T XorValue)
	{
		auto Result = FPlatformAtomics::InterlockedXor((volatile TUnderlyingIntegerType_T<T>*)Element, (TUnderlyingIntegerType_T<T>)XorValue);
		return *(const T*)&Result;
	}

	template <typename T, bool bIsVoidPointer, bool bIsIntegral, bool bCanUsePlatformAtomics>
	struct TAtomicBaseType
	{
		using Type = TAtomicBase_Basic<T>;
	};

	template <typename T>
	struct TAtomicBaseType<T, false, false, false>
	{
		static_assert(sizeof(T) == 0, "TAtomic of this size are not currently supported");
		using Type = TAtomicBase_Mutex<T>;
	};

	template <typename T>
	struct TAtomicBaseType<T*, false, false, true>
	{
		using Type = TAtomicBase_Pointer<T*>;
	};

	template <typename T>
	struct TAtomicBaseType<T, false, true, true>
	{
		using Type = TAtomicBase_Integral<T>;
	};

	template <typename T, bool bIsVoidPointer = TIsVoidPointer<T>::Value, bool bIsIntegral = TIsIntegral<T>::Value, bool bCanUsePlatformAtomics = TIsSupportedSize<sizeof(T)>::Value>
	using TAtomicBaseType_T = typename TAtomicBaseType<T, bIsVoidPointer, bIsIntegral, bCanUsePlatformAtomics>::Type;
}

// Basic storage and implementation - only allows getting and setting via platform atomics.
template <typename T>
struct 
	TAtomicBase_Basic
{
public:
	/**
	 * Gets a copy of the current value of the element.
	 *
	 * @param  Order  The memory ordering semantics to apply to the operation.
	 *
	 * @return The current value.
	 */
	FORCEINLINE T Load(EMemoryOrder Order = EMemoryOrder::SequentiallyConsistent) const
	{
		return Element.load(ToStd(Order));
	}

	/**
	 * Sets the element to a specific value.
	 *
	 * @param  Value  The value to set the element to.
	 * @param  Order  The memory ordering semantics to apply to the operation.
	 */
	FORCEINLINE void Store(T Value, EMemoryOrder Order = EMemoryOrder::SequentiallyConsistent)
	{
		Element.store(Value, ToStd(Order));
	}

	/**
	 * Sets the element to a specific value, returning a copy of the previous value.
	 *
	 * @param  Value  The value to set the element to.
	 *
	 * @return A copy of the previous value.
	 */
	FORCEINLINE T Exchange(T Value)
	{
		return Element.exchange(Value);
	}

	/**
	 * Compares the element with an expected value and, only if comparison succeeds, assigns the element to a new value.
	 * The previous value is copied into Expected in any case.
	 *
	 * @param  Expected  The value to compare to the element, and will hold the existing value of the element after returning.
	 * @param  Value     The value to assign, only if Expected matches the element.
	 *
	 * @return Whether the element compared equal to Expected.
	 *
	 * @note It is equivalent to this in non-atomic terms:
	 *
	 * bool CompareExchange(T& Expected, T Value)
	 * {
	 *     bool bResult = this->Element == Expected;
	 *     Expected = this->Element;
	 *     if (bResult)
	 *     {
	 *         this->Element = Value;
	 *     }
	 *     return bResult;
	 * }
	 */
	FORCEINLINE bool CompareExchange(T& Expected, T Value)
	{
		return Element.compare_exchange_strong(Expected, Value);
	}

protected:
	TAtomicBase_Basic() = default;

	constexpr TAtomicBase_Basic(T Value)
		: Element(Value)
	{
	}

	std::atomic<T> Element;

private:
	static std::memory_order ToStd(EMemoryOrder Order)
	{
		std::memory_order Map[(int)EMemoryOrder::Count] = { std::memory_order_relaxed, std::memory_order_seq_cst };
		return Map[(int)Order];
	}
};

// TODO : basic storage and getting and setting, but the element is protected by a mutex instead of using platform atomics.
// Also, we may not want to do this ever.
template <typename T>
struct TAtomicBase_Mutex;

// Arithmetic atomic implementation - used by both pointers and integral types in addition to getting and setting.
template <typename T, typename DiffType>
struct TAtomicBase_Arithmetic : public TAtomicBase_Basic<T>
{
public:
	/**
	 * Performs ++Element, returning a copy of the new value of the element.
	 *
	 * @return A copy of the new, incremented value.
	 */
	FORCEINLINE T operator++()
	{
		return this->Element.fetch_add(1) + 1;
	}

	/**
	 * Performs Element++, returning a copy of the previous value of the element.
	 *
	 * @return A copy of the previous, unincremented value.
	 */
	FORCEINLINE T operator++(int)
	{
		return this->Element.fetch_add(1);
	}

	/**
	 * Performs Element += Value, returning a copy of the new value of the element.
	 *
	 * @param  Value  The value to add to the element.
	 *
	 * @return A copy of the new, incremented value.
	 */
	FORCEINLINE T operator+=(DiffType Value)
	{
		return this->Element.fetch_add(Value) + Value;
	}

	/**
	 * Performs --Element, returning a copy of the new value of the element.
	 *
	 * @return A copy of the new, decremented value.
	 */
	FORCEINLINE T operator--()
	{
		return this->Element.fetch_sub(1) - 1;
	}

	/**
	 * Performs Element--, returning a copy of the previous value of the element.
	 *
	 * @return A copy of the previous, undecremented value.
	 */
	FORCEINLINE T operator--(int)
	{
		return this->Element.fetch_sub(1);
	}

	/**
	 * Performs Element -= Value, returning a copy of the new value of the element.
	 *
	 * @param  Value  The value to subtract from the element.
	 *
	 * @return A copy of the new, decremented value.
	 */
	FORCEINLINE T operator-=(DiffType Value)
	{
		return this->Element.fetch_sub(Value) - Value;
	}

	/**
	 * Increments Element and returns a copy of the previous value of the element.
	 *
	 * @return A copy of the previous, unincremented value.
	 */
	FORCEINLINE T IncrementExchange()
	{
		return this->Element.fetch_add(1);
	}

	/**
	 * Decrements Element and returns a copy of the previous value of the element.
	 *
	 * @return A copy of the previous, undecremented value.
	 */
	FORCEINLINE T DecrementExchange()
	{
		return this->Element.fetch_sub(1);
	}

	/**
	 * Adds Value to Element and returns a copy of the previous value of the element.
	 *
	 * @param  Value  The value to add to the element.
	 *
	 * @return A copy of the previous, unincremented value.
	 */
	FORCEINLINE T AddExchange(DiffType Value)
	{
		return this->Element.fetch_add(Value);
	}

	/**
	 * Subtracts Value from Element and returns a copy of the previous value of the element.
	 *
	 * @param  Value  The value to subtract from the element.
	 *
	 * @return A copy of the previous, undecremented value.
	 */
	FORCEINLINE T SubExchange(DiffType Value)
	{
		return this->Element.fetch_sub(Value);
	}

protected:
	TAtomicBase_Arithmetic() = default;

	constexpr TAtomicBase_Arithmetic(T Value)
		: TAtomicBase_Basic<T>(Value)
	{
	}
};

// Pointer atomic implementation - allows arithmetic with PTRINT.
template <typename T>
struct TAtomicBase_Pointer : public TAtomicBase_Arithmetic<T, PTRINT>
{
protected:
	TAtomicBase_Pointer() = default;

	constexpr TAtomicBase_Pointer(T Value)
		: TAtomicBase_Arithmetic<T, PTRINT>(Value)
	{
	}
};

// Integral atomic implementation - allows arithmetic and bitwise operations.
template <typename T>
struct TAtomicBase_Integral : public TAtomicBase_Arithmetic<T, T>
{
public:
	/**
	 * Performs Element &= Value, returning a copy of the new value of the element.
	 *
	 * @param  Value  The value to and the element with.
	 *
	 * @return A copy of the new value.
	 */
	FORCEINLINE T operator&=(const T Value)
	{
		return this->Element.fetch_and(Value) & Value;
	}

	/**
	 * Performs Element |= Value, returning a copy of the new value of the element.
	 *
	 * @param  Value  The value to or the element with.
	 *
	 * @return A copy of the new value.
	 */
	FORCEINLINE T operator|=(const T Value)
	{
		return this->Element.fetch_or(Value) | Value;
	}

	/**
	 * Performs Element ^= Value, returning a copy of the new value of the element.
	 *
	 * @param  Value  The value to xor the element with.
	 *
	 * @return A copy of the new value.
	 */
	FORCEINLINE T operator^=(const T Value)
	{
		return this->Element.fetch_xor(Value) ^ Value;
	}

	/**
	 * Performs Element &= Value, returning a copy of the previous value of the element.
	 *
	 * @param  Value  The value to and the element with.
	 *
	 * @return A copy of the previous value.
	 */
	FORCEINLINE T AndExchange(const T Value)
	{
		return this->Element.fetch_and(Value);
	}

	/**
	 * Performs Element |= Value, returning a copy of the previous value of the element.
	 *
	 * @param  Value  The value to or the element with.
	 *
	 * @return A copy of the previous value.
	 */
	FORCEINLINE T OrExchange(const T Value)
	{
		return this->Element.fetch_or(Value);
	}

	/**
	 * Performs Element ^= Value, returning a copy of the previous value of the element.
	 *
	 * @param  Value  The value to xor the element with.
	 *
	 * @return A copy of the previous value.
	 */
	FORCEINLINE T XorExchange(const T Value)
	{
		return this->Element.fetch_xor(Value);
	}


protected:
	TAtomicBase_Integral() = default;

	constexpr TAtomicBase_Integral(T Value)
		: TAtomicBase_Arithmetic<T, T>(Value)
	{
	}
};

/**
 * DEPRECATED! UE atomics are not maintained and potentially will be physically deprecated. Use std::atomic<T> for new code
 *
 * Atomic object wrapper class which wraps an element of T.  This allows the following benefits:
 *
 * - Changes made to the element on one thread can never be observed as a partial state by other threads. (atomicity)
 * - Memory accesses within a thread are not reordered across any access of the element. (acquire/release semantics)
 * - There is a single, visible, global order of atomic accesses throughout the system. (sequential consistency)
 */
template <typename T>
class TAtomic final : public UE::Core::Private::Atomic::TAtomicBaseType_T<T>
{
	static_assert(TIsTrivial<T>::Value, "TAtomic is only usable with trivial types");

public:
	using ElementType = T;

	/**
	 * Default initializes the element type.  NOTE: This will leave the value uninitialized if it has no constructor.
	 */
	FORCEINLINE TAtomic() = default;

	/**
	 * Initializes the element type with the given argument types.
	 *
	 * @param  Arg  The value to initialize the element counter to
	 *
	 * @note  This constructor is not atomic.
	 */
	constexpr TAtomic(T Arg)
		: UE::Core::Private::Atomic::TAtomicBaseType_T<T>(Arg)
	{
	}

	/**
	 * Gets a copy of the current value of the element.
	 *
	 * @return The current value.
	 */
	FORCEINLINE operator T() const
	{
		return this->Load();
	}

	/**
	 * Sets the element to a specific value, returning a copy of the value.
	 *
	 * @param  Value  The value to set the element to.
	 *
	 * @return A copy of Value.
	 */
	FORCEINLINE T operator=(T Value)
	{
		this->Exchange(Value);
		return Value;
	}

private:
	// Non-copyable and non-movable
	TAtomic(TAtomic&&) = delete;
	TAtomic(const TAtomic&) = delete;
	TAtomic& operator=(TAtomic&&) = delete;
	TAtomic& operator=(const TAtomic&) = delete;
};
