// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/CopyQualifiersFromTo.h"

/**
 * Copies the cv-qualifiers and references from one type to another, e.g.:
 *
 * TCopyQualifiersFromTo_T<const T1, T2> == const T2
 * TCopyQualifiersFromTo_T<T1&, const T2> == const T2&
 */
template <typename From, typename To> struct TCopyQualifiersAndRefsFromTo               { using Type = TCopyQualifiersFromTo_T<From, To>;   };
template <typename From, typename To> struct TCopyQualifiersAndRefsFromTo<From,   To& > { using Type = TCopyQualifiersFromTo_T<From, To>&;  };
template <typename From, typename To> struct TCopyQualifiersAndRefsFromTo<From,   To&&> { using Type = TCopyQualifiersFromTo_T<From, To>&&; };
template <typename From, typename To> struct TCopyQualifiersAndRefsFromTo<From&,  To  > { using Type = TCopyQualifiersFromTo_T<From, To>&;  };
template <typename From, typename To> struct TCopyQualifiersAndRefsFromTo<From&,  To& > { using Type = TCopyQualifiersFromTo_T<From, To>&;  };
template <typename From, typename To> struct TCopyQualifiersAndRefsFromTo<From&,  To&&> { using Type = TCopyQualifiersFromTo_T<From, To>&;  };
template <typename From, typename To> struct TCopyQualifiersAndRefsFromTo<From&&, To  > { using Type = TCopyQualifiersFromTo_T<From, To>&&; };
template <typename From, typename To> struct TCopyQualifiersAndRefsFromTo<From&&, To& > { using Type = TCopyQualifiersFromTo_T<From, To>&;  };
template <typename From, typename To> struct TCopyQualifiersAndRefsFromTo<From&&, To&&> { using Type = TCopyQualifiersFromTo_T<From, To>&&; };

template <typename From, typename To>
using TCopyQualifiersAndRefsFromTo_T = typename TCopyQualifiersAndRefsFromTo<From, To>::Type;
