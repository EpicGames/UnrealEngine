// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Copies the cv-qualifiers from one type to another, e.g.:
 *
 * TCopyQualifiersFromTo_T<const    T1,       T2> == const T2
 * TCopyQualifiersFromTo_T<volatile T1, const T2> == const volatile T2
 */
template <typename From, typename To> struct TCopyQualifiersFromTo                          { typedef                To Type; };
template <typename From, typename To> struct TCopyQualifiersFromTo<const          From, To> { typedef const          To Type; };
template <typename From, typename To> struct TCopyQualifiersFromTo<      volatile From, To> { typedef       volatile To Type; };
template <typename From, typename To> struct TCopyQualifiersFromTo<const volatile From, To> { typedef const volatile To Type; };

template <typename From, typename To>
using TCopyQualifiersFromTo_T = typename TCopyQualifiersFromTo<From, To>::Type;
