// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/RemoveReference.h"
#include <type_traits>

namespace UE::Core::Private::Decay
{
	template <typename T>
	struct TDecayNonReference
	{
		using Type = std::remove_cv_t<T>;
	};

	template <typename T>
	struct TDecayNonReference<T[]>
	{
		typedef T* Type;
	};

	template <typename T, uint32 N>
	struct TDecayNonReference<T[N]>
	{
		typedef T* Type;
	};

	template <typename RetType, typename... Params>
	struct TDecayNonReference<RetType(Params...)>
	{
		typedef RetType (*Type)(Params...);
	};
}

/**
 * Returns the decayed type of T, meaning it removes all references, qualifiers and
 * applies array-to-pointer and function-to-pointer conversions.
 *
 * http://en.cppreference.com/w/cpp/types/decay
 */
template <typename T>
struct TDecay
{
	typedef typename UE::Core::Private::Decay::TDecayNonReference<typename TRemoveReference<T>::Type>::Type Type;
};
