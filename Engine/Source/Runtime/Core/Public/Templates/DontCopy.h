// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// Can be used as a wrapper over non-copyable objects (i.e. FCriticalSection) if you still want the class containing your
// object to keep its copyable property without compromising integrity of the object that doesn't support being copied.
// It makes each TDontCopy member of a class an exclusive entity of that class.

template <typename T>
struct TDontCopy
{
	TDontCopy() = default;
	~TDontCopy() = default;
	TDontCopy(TDontCopy&&) {}
	TDontCopy(const TDontCopy&) {}
	TDontCopy& operator=(TDontCopy&&) { return *this; }
	TDontCopy& operator=(const TDontCopy&) { return *this; }

	T& Get() { return Value; }
	const T& Get() const { return Value; }

	T* operator->()       { return &Value; }
	const T* operator->() const { return &Value; }
	
	T& operator*() { return Value; }
	const T& operator*() const { return Value; }

private:
    T Value;
};
