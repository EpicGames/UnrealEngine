// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Includes a function in an overload set if the predicate is true.  It should be used similarly to this:
 *
 * // This function will only be instantiated if SomeTrait<T>::Value is true for a particular T
 * template <typename T>
 * typename TEnableIf<SomeTrait<T>::Value, ReturnType>::Type Function(const T& Obj)
 * {
 *     ...
 * }
 *
 * ReturnType is the real return type of the function.
 */
template <bool Predicate, typename Result = void>
class TEnableIf;

template <typename Result>
class TEnableIf<true, Result>
{
public:
	using type = Result;
	using Type = Result;
};

template <typename Result>
class TEnableIf<false, Result>
{ };


/**
 * This is a variant of the above that will determine the return type 'lazily', i.e. only if the function is enabled.
 * This is useful when the return type isn't necessarily legal code unless the enabling condition is true.
 *
 * // This function will only be instantiated if SomeTrait<T>::Value is true for a particular T.
 * // The function's return type is typename Transform<T>::Type.
 * template <typename T>
 * typename TLazyEnableIf<SomeTrait<T>::Value, Transform<T>>::Type Function(const T& Obj)
 * {
 *     ...
 * }
 *
 * See boost::lazy_enable_if for more details.
 */
template <bool Predicate, typename Func>
class TLazyEnableIf;

template <typename Func>
class TLazyEnableIf<true, Func>
{
public:
	using type = typename Func::Type;
	using Type = typename Func::Type;
};

template <typename Func>
class TLazyEnableIf<false, Func>
{
};
