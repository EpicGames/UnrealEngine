// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Concepts/SameAs.h"
#include "UnrealTemplate.h"

/**
 * Binary predicate class for performing equality comparisons.  Assumes == operator is defined for the template type.
 *
 * See: https://en.cppreference.com/w/cpp/utility/functional/equal_to
 *
 * TEqualTo should not be specialized - an appropriate operator== should be added.
 */
template <typename T = void>
struct TEqualTo
{
	[[nodiscard]] UE_REWRITE constexpr bool operator()(const T& Lhs, const T& Rhs) const
	{
		if constexpr (requires { { Lhs == Rhs } -> UE::CSameAs<bool>; })
		{
			return Lhs == Rhs;
		}
		else
		{
			static_assert(sizeof(T) == 0, "Trying to use TEqualTo<T> where T doesn't have an appropriate operator== overload. Please add bool operator==(T, T), do not specialize TEqualTo.");
		} 
	} //-V591
};

template <>
struct TEqualTo<void>
{
	template <typename T, typename U>
	[[nodiscard]] UE_REWRITE constexpr bool operator()(T&& Lhs, U&& Rhs) const
	{
		if constexpr (requires { { Forward<T>(Lhs) == Forward<U>(Rhs) } -> UE::CSameAs<bool>; })
		{
			return Forward<T>(Lhs) == Forward<U>(Rhs);
		}
		else
		{
			static_assert(sizeof(T) == 0, "Trying to use TEqualTo<void> with types without an appropriate operator== overload. Please add bool operator==(T, U), do not specialize TEqualTo.");
		} 
	} //-V591
};
