// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "AutoRTFM.h"
#include "Concepts/NotCVRefTo.h"
#include "Misc/AssertionMacros.h"
#include "Misc/IntrusiveUnsetOptionalState.h"
#include "HAL/UnrealMemory.h"
#include "Templates/FunctionFwd.h" // IWYU pragma: export
#include "Templates/UnrealTypeTraits.h"
#include "Templates/Invoke.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/Requires.h"
#include "Math/UnrealMathUtility.h"
#include <new> // IWYU pragma: export
#include <type_traits>

// Disable visualization hack for shipping or test builds.
#if defined(UE_ENABLE_TFUNCTIONREF_VISUALIZATION) && UE_ENABLE_TFUNCTIONREF_VISUALIZATION
	COMPILE_WARNING("TFunction visualization has been removed - please undefine UE_ENABLE_TFUNCTIONREF_VISUALIZATION.")
#endif

#if !defined(NUM_TFUNCTION_INLINE_BYTES) || NUM_TFUNCTION_INLINE_BYTES == 0
	#define TFUNCTION_USES_INLINE_STORAGE 0
#else
	#define TFUNCTION_USES_INLINE_STORAGE 1
	#define TFUNCTION_INLINE_SIZE         NUM_TFUNCTION_INLINE_BYTES
	#define TFUNCTION_INLINE_ALIGNMENT    16
#endif

/**
 * Private implementation details of TFunction and TFunctionRef.
 */
namespace UE::Core::Private::Function
{
	template <typename T, bool bUnique, bool bOnHeap>
	struct TFunction_OwnedObject;

	struct FFunctionStorage;

	template <bool bUnique>
	struct TFunctionStorage;

	/**
	 * Common interface to a callable object owned by TFunction.
	 */
	struct IFunction_OwnedObject
	{
		/**
		 * Creates a copy of itself into the storage and returns a pointer to the new object within it.
		 */
		virtual void* CloneToEmptyStorage(FFunctionStorage* Storage) const = 0;

		/**
		 * Returns the address of the object.
		 */
		virtual void* GetAddress() = 0;

		/**
		 * Destructor.
		 */
		virtual void Destroy() = 0;

		/**
		 * Destructor.
		 */
		virtual ~IFunction_OwnedObject() = default;
	};

	template <typename T, bool bUnique, bool bOnHeap>
	struct TFunction_OwnedObject : public IFunction_OwnedObject
	{
		template <typename ArgType>
		explicit TFunction_OwnedObject(ArgType&& Arg)
			: Obj(Forward<ArgType>(Arg))
		{
		}

		virtual void* GetAddress() override
		{
			return &Obj;
		}

		void* CloneToEmptyStorage(FFunctionStorage* UntypedStorage) const override;

		virtual void Destroy() override
		{
			if constexpr (bOnHeap)
			{
				void* This = this;
				this->~TFunction_OwnedObject();
				FMemory::Free(This);
			}
			else
			{
				this->~TFunction_OwnedObject();
			}
		}

		~TFunction_OwnedObject() override
		{
			// It is not necessary to define this destructor but MSVC will
			// erroneously issue warning C5046 without it.
		}

		T Obj;
	};

	template <typename T>
	FORCEINLINE bool IsBound(const T& Func)
	{
		if constexpr (std::is_pointer_v<T> || std::is_member_pointer_v<T> || ::TIsTFunction<T>::Value)
		{
			// Function pointers, data member pointers, member function pointers and TFunctions
			// can all be null/unbound, so test them using their boolean state.
			return !!Func;
		}
		else
		{
			// We can't tell if any other generic callable can be invoked, so just assume they can be.
			return true;
		}
	}

	struct AUTORTFM_INFER FFunctionStorage
	{
		constexpr static bool bCanBeNull = true;

		FFunctionStorage()
			: HeapAllocation(nullptr)
		{
		}

		FFunctionStorage(FFunctionStorage&& Other)
			: HeapAllocation(Other.HeapAllocation)
		{
			Other.HeapAllocation = nullptr;
			#if TFUNCTION_USES_INLINE_STORAGE
				FMemory::Memcpy(&InlineAllocation, &Other.InlineAllocation, sizeof(InlineAllocation));
			#endif
		}

		FFunctionStorage(const FFunctionStorage& Other) = delete;
		FFunctionStorage& operator=(FFunctionStorage&& Other) = delete;
		FFunctionStorage& operator=(const FFunctionStorage& Other) = delete;

		void BindCopy(const FFunctionStorage& Other)
		{
			Other.GetBoundObject()->CloneToEmptyStorage(this);
		}

		IFunction_OwnedObject* GetBoundObject() const
		{
			IFunction_OwnedObject* Result = (IFunction_OwnedObject*)HeapAllocation;
			#if TFUNCTION_USES_INLINE_STORAGE
				if (!Result)
				{
					Result = (IFunction_OwnedObject*)&InlineAllocation;
				}
			#endif

			return Result;
		}

		/**
		 * Returns a pointer to the callable object - needed by TFunctionRefBase.
		 */
		void* GetPtr() const
		{
		#if TFUNCTION_USES_INLINE_STORAGE
			IFunction_OwnedObject* Owned = (IFunction_OwnedObject*)HeapAllocation;
			if (!Owned)
			{
				Owned = (IFunction_OwnedObject*)&InlineAllocation;
			}

			return Owned->GetAddress();
		#else
			return ((IFunction_OwnedObject*)HeapAllocation)->GetAddress();
		#endif
		}

		/**
		 * Destroy any owned bindings - called by TFunctionRefBase only if Bind() or BindCopy() was called.
		 */
		void Unbind()
		{
			IFunction_OwnedObject* Owned = GetBoundObject();
			Owned->Destroy();
		}

		#if TFUNCTION_USES_INLINE_STORAGE
			// Inline storage for an owned object
			alignas(TFUNCTION_INLINE_ALIGNMENT) uint8 InlineAllocation[TFUNCTION_INLINE_SIZE];
		#endif

		void* HeapAllocation;
	};

	template <bool bUnique>
	struct TFunctionStorage : FFunctionStorage
	{
		TFunctionStorage() = default;

		TFunctionStorage(FFunctionStorage&& Other)
			: FFunctionStorage(MoveTemp(Other))
		{
		}

		template <typename FunctorType>
		std::decay_t<FunctorType>* Bind(FunctorType&& InFunc)
		{
			using DecayedFunctorType = std::decay_t<FunctorType>;

			if (!IsBound(InFunc))
			{
				return nullptr;
			}

#if TFUNCTION_USES_INLINE_STORAGE
			constexpr bool bOnHeap = sizeof(TFunction_OwnedObject<DecayedFunctorType, bUnique, false>) > TFUNCTION_INLINE_SIZE;
#else
			constexpr bool bOnHeap = true;
#endif

			using OwnedType = TFunction_OwnedObject<DecayedFunctorType, bUnique, bOnHeap>;

			void* NewAlloc;
#if TFUNCTION_USES_INLINE_STORAGE
			if constexpr (!bOnHeap)
			{
				NewAlloc = &InlineAllocation;
			}
			else
#endif
			{
				NewAlloc = FMemory::Malloc(sizeof(OwnedType), alignof(OwnedType));
				HeapAllocation = NewAlloc;
			}

			CA_ASSUME(NewAlloc);
			auto* NewOwned = ::new (NewAlloc) OwnedType(Forward<FunctorType>(InFunc));
			return &NewOwned->Obj;
		}
	};

	template <typename T, bool bUnique, bool bOnHeap>
	void* TFunction_OwnedObject<T, bUnique, bOnHeap>::CloneToEmptyStorage(FFunctionStorage* UntypedStorage) const
	{
		if constexpr (bUnique)
		{
			// Should never get here - copy functions are deleted for TUniqueFunction
			check(false);
			return nullptr;
		}
		else
		{
			TFunctionStorage<false>& Storage = *static_cast<TFunctionStorage<false>*>(UntypedStorage);

			void* NewAlloc;
#if TFUNCTION_USES_INLINE_STORAGE
			if constexpr (!bOnHeap)
			{
				NewAlloc = &Storage.InlineAllocation;
			}
			else
#endif
			{
				NewAlloc = FMemory::Malloc(sizeof(TFunction_OwnedObject), alignof(TFunction_OwnedObject));
				Storage.HeapAllocation = NewAlloc;
				CA_ASSUME(NewAlloc);
			}

			auto* NewOwned = ::new (NewAlloc) TFunction_OwnedObject(this->Obj);

			return &NewOwned->Obj;
		}
	}

	/**
	 * A class which is used to instantiate the code needed to call a bound function.
	 */
	template <typename Functor, typename Ret, typename... ParamTypes>
	struct TFunctionRefCaller
	{
		static Ret Call(void* Obj, ParamTypes&... Params)
		{
			if constexpr (std::is_void_v<Ret>)
			{
				::Invoke(*(Functor*)Obj, Forward<ParamTypes>(Params)...);
			}
			else
			{
				return ::Invoke(*(Functor*)Obj, Forward<ParamTypes>(Params)...);
			}
		}
	};

#if DO_CHECK
	// Move all of the assert code out of line.
	// This takes a void* to avoid multiple template instantiations - the caller should
	// explicitly cast the Callable function pointer to void* to call this function.
	CORE_API void CheckCallable(void* Callable);
#endif

	/**
	 * A class which defines an operator() which will own storage of a callable and invoke the TFunctionRefCaller::Call function on it.
	 */
	template <typename StorageType, typename FuncType>
	struct TFunctionRefBase;

	template <typename StorageType, typename Ret, typename... ParamTypes>
	struct TFunctionRefBase<StorageType, Ret (ParamTypes...)>
	{
		template <typename OtherStorageType, typename OtherFuncType>
		friend struct TFunctionRefBase;

		TFunctionRefBase() = default;

		TFunctionRefBase(TFunctionRefBase&& Other)
			: Callable(Other.Callable)
			, Storage (MoveTemp(Other.Storage))
		{
			static_assert(StorageType::bCanBeNull, "Unable to move non-nullable storage");

			if (Callable)
			{
				Other.Callable = nullptr;
			}
		}

		template <typename OtherStorage>
		TFunctionRefBase(TFunctionRefBase<OtherStorage, Ret (ParamTypes...)>&& Other)
			: Callable(Other.Callable)
			, Storage (MoveTemp(Other.Storage))
		{
			static_assert(OtherStorage::bCanBeNull, "Unable to move from non-nullable storage");
			static_assert(StorageType::bCanBeNull,  "Unable to move into non-nullable storage");

			if (Callable)
			{
				Other.Callable = nullptr;
			}
		}

		template <typename OtherStorage>
		TFunctionRefBase(const TFunctionRefBase<OtherStorage, Ret (ParamTypes...)>& Other)
			: Callable(Other.Callable)
		{
			if constexpr (OtherStorage::bCanBeNull)
			{
				static_assert(StorageType::bCanBeNull, "Unable to copy from nullable storage into non-nullable storage");

				if (!Callable)
				{
					return;
				}
			}

			Storage.BindCopy(Other.Storage);
		}

		TFunctionRefBase(const TFunctionRefBase& Other)
			: Callable(Other.Callable)
		{
			if constexpr (StorageType::bCanBeNull)
			{
				if (!Callable)
				{
					return;
				}
			}

			Storage.BindCopy(Other.Storage);
		}

		template <UE::CNotCVRefTo<TFunctionRefBase> FunctorType>
		TFunctionRefBase(FunctorType&& InFunc)
		{
			auto* Binding = Storage.Bind(Forward<FunctorType>(InFunc));

			if constexpr (StorageType::bCanBeNull)
			{
				if (!Binding)
				{
					return;
				}
			}

			using DecayedFunctorType = typename ::TRemovePointer<decltype(Binding)>::Type;

			Callable = &TFunctionRefCaller<DecayedFunctorType, Ret, ParamTypes...>::Call;
		}

		TFunctionRefBase& operator=(TFunctionRefBase&&) = delete;
		TFunctionRefBase& operator=(const TFunctionRefBase&) = delete;

		UE_DEPRECATED(5.7, "CheckCallable should not be called - use check(Func) instead, and for a TFunctionRef there's no need to check.")
		FORCENOINLINE void CheckCallable() const
		{
			checkf(Callable, TEXT("Attempting to call an unbound TFunction!"));
		}

		Ret operator()(ParamTypes... Params) const
		{
#if DO_CHECK
			if constexpr (StorageType::bCanBeNull)
			{
				UE::Core::Private::Function::CheckCallable((void*)Callable);
			}
#endif
			return Callable(Storage.GetPtr(), Params...);
		}

		~TFunctionRefBase()
		{
			if constexpr (StorageType::bCanBeNull)
			{
				if (!Callable)
				{
					return;
				}
			}
			Storage.Unbind();
		}

		/**
		 * Removes any bound callable from the function, restoring it to the default 'empty' state.
		 */
		void Reset()
		{
			if (Callable)
			{
				Storage.Unbind();
				Callable = nullptr;
			}
		}

	protected:
		bool IsSet() const
		{
			// Normally we'd assert that bCanBeNull here because it should always be true, but we reuse this
			// function to test that a `TOptional<TFunctionRef>` with an intrusive state is unset.

			return !!Callable;
		}

	private:
		// A pointer to a function which invokes the call operator on the callable object
		Ret (*Callable)(void*, ParamTypes&...) = nullptr;

		StorageType Storage;
	};

	struct FFunctionRefStoragePolicy
	{
		constexpr static bool bCanBeNull = false;

		template <typename FunctorType>
		std::remove_reference_t<FunctorType>* Bind(FunctorType&& InFunc)
		{
			checkf(IsBound(InFunc), TEXT("Cannot bind a null/unbound callable to a TFunctionRef"));

			Ptr = (void*)&InFunc;
			return &InFunc;
		}

		void BindCopy(const FFunctionRefStoragePolicy& Other)
		{
			Ptr = Other.Ptr;
		}

		/**
		 * Returns a pointer to the callable object - needed by TFunctionRefBase.
		 */
		void* GetPtr() const
		{
			return Ptr;
		}

		/**
		 * Destroy any owned bindings - called by TFunctionRefBase only if Bind() or BindCopy() was called.
		 */
		void Unbind() const
		{
			// FunctionRefs don't own their binding - do nothing
		}

	private:
		// A pointer to the callable object
		void* Ptr = nullptr;
	};

	template <typename FunctorType> auto ResolveFuncPtrTypeIfPossible(FunctorType&&, int) -> decltype(+std::declval<FunctorType>());
	template <typename FunctorType> auto ResolveFuncPtrTypeIfPossible(FunctorType&&, ...) -> FunctorType&&;

	template <typename FunctorType>
	using TFuncPtrTypeIfPossible_T = decltype(ResolveFuncPtrTypeIfPossible(std::declval<FunctorType&&>(), 0));
}

/**
 * TFunctionRef<FuncType>
 *
 * A class which represents a reference to something callable.  The important part here is *reference* - if
 * you bind it to a lambda and the lambda goes out of scope, you will be left with an invalid reference.
 *
 * FuncType represents a function type and so TFunctionRef should be defined as follows:
 *
 * // A function taking a string and float and returning int32.  Parameter names are optional.
 * TFunctionRef<int32 (const FString& Name, float Scale)>
 *
 * If you also want to take ownership of the callable thing, e.g. you want to return a lambda from a
 * function, you should use TFunction.  TFunctionRef does not concern itself with ownership because it's
 * intended to be faster than TFunction.  A TFunctionRef is lightweight and meant to be passed and copied
 * by value; there is no benefit to using MoveTemp.
 *
 * TFunctionRef is most useful when you want to parameterize a function with some caller-defined code
 * without making it a template.
 *
 * Example:
 *
 * // Something.h
 * void DoSomethingWithConvertingStringsToInts(TFunctionRef<int32 (const FString& Str)> Convert);
 *
 * // Something.cpp
 * void DoSomethingWithConvertingStringsToInts(TFunctionRef<int32 (const FString& Str)> Convert)
 * {
 *     for (const FString& Str : SomeBunchOfStrings)
 *     {
 *         int32 Int = Convert(Str);
 *         DoSomething(Int);
 *     }
 * }
 *
 * // SomewhereElse.cpp
 * #include "Something.h"
 *
 * void Func()
 * {
 *     // First do something using string length
 *     DoSomethingWithConvertingStringsToInts([](const FString& Str) {
 *         return Str.Len();
 *     });
 *
 *     // Then do something using string conversion
 *     DoSomethingWithConvertingStringsToInts([](const FString& Str) {
 *         int32 Result;
 *         TTypeFromString<int32>::FromString(Result, *Str);
 *         return Result;
 *     });
 * }
 */
template <typename Ret, typename... ParamTypes>
class TFunctionRef<Ret(ParamTypes...)> final : public UE::Core::Private::Function::TFunctionRefBase<UE::Core::Private::Function::FFunctionRefStoragePolicy, Ret(ParamTypes...)>
{
	using Super = UE::Core::Private::Function::TFunctionRefBase<UE::Core::Private::Function::FFunctionRefStoragePolicy, Ret(ParamTypes...)>;

public:
	/**
	 * Constructor which binds a TFunctionRef to a callable object.
	 */
	template <typename FunctorType>
		requires (!::TIsTFunctionRef<std::decay_t<FunctorType>>::Value && std::is_invocable_r_v<Ret, std::decay_t<FunctorType>, ParamTypes...>)
	TFunctionRef(FunctorType&& InFunc UE_LIFETIMEBOUND)
		: Super(Forward<FunctorType>(InFunc))
	{
		// This constructor is disabled for TFunctionRef types so it isn't incorrectly selected as copy/move constructors.

		// Unlike TFunction and TUniqueFunction, we do not coerce the function type to a pointer here because the functionref would
		// end up pointing to the temporary pointer we created on the stack.
	}

	/////////////////////////////////////////////////////
	// Start - intrusive TOptional<TFunctionRef> state //
	/////////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TFunctionRef;

	explicit TFunctionRef(FIntrusiveUnsetOptionalState)
	{
	}
	bool operator==(FIntrusiveUnsetOptionalState) const
	{
		return !Super::IsSet();
	}
	///////////////////////////////////////////////////
	// End - intrusive TOptional<TFunctionRef> state //
	///////////////////////////////////////////////////

	TFunctionRef(const TFunctionRef&) = default;

	// We delete the assignment operators because we don't want it to be confused with being related to
	// regular C++ reference assignment - i.e. calling the assignment operator of whatever the reference
	// is bound to - because that's not what TFunctionRef does, nor is it even capable of doing that.
	TFunctionRef& operator=(const TFunctionRef&) const = delete;
	~TFunctionRef() = default;
};

/**
 * TFunction<FuncType>
 *
 * A class which represents a copy of something callable.  FuncType represents a function type and so
 * TFunction should be defined as follows:
 *
 * // A function taking a string and float and returning int32.  Parameter names are optional.
 * TFunction<int32 (const FString& Name, float Scale)>
 *
 * Unlike TFunctionRef, this object is intended to be used like a UE version of std::function.  That is,
 * it takes a copy of whatever is bound to it, meaning you can return it from functions and store them in
 * objects without caring about the lifetime of the original object being bound.
 *
 * Example:
 *
 * // Something.h
 * TFunction<FString (int32)> GetTransform();
 *
 * // Something.cpp
 * TFunction<FString (int32)> GetTransform(const FString& Prefix)
 * {
 *     // Squares number and returns it as a string with the specified prefix
 *     return [=](int32 Num) {
 *         return Prefix + TEXT(": ") + TTypeToString<int32>::ToString(Num * Num);
 *     };
 * }
 *
 * // SomewhereElse.cpp
 * #include "Something.h"
 *
 * void Func()
 * {
 *     TFunction<FString (int32)> Transform = GetTransform(TEXT("Hello"));
 *
 *     FString Result = Transform(5); // "Hello: 25"
 * }
 */
template <typename Ret, typename... ParamTypes>
class TFunction<Ret(ParamTypes...)> final : public UE::Core::Private::Function::TFunctionRefBase<UE::Core::Private::Function::TFunctionStorage<false>, Ret(ParamTypes...)>
{
	using Super = UE::Core::Private::Function::TFunctionRefBase<UE::Core::Private::Function::TFunctionStorage<false>, Ret(ParamTypes...)>;

public:
	/**
	 * Default constructor.
	 */
	TFunction(TYPE_OF_NULLPTR = nullptr)
	{
	}

	/**
	 * Constructor which binds a TFunction to any function object.
	 */
	template <typename FunctorType>
		requires (!::TIsTFunction<std::decay_t<FunctorType>>::Value && std::is_invocable_r_v<Ret, std::decay_t<FunctorType>, ParamTypes...>)
	TFunction(FunctorType&& InFunc)
		: Super(static_cast<UE::Core::Private::Function::TFuncPtrTypeIfPossible_T<FunctorType>>(InFunc))
	{
		// This constructor is disabled for TFunction types so it isn't incorrectly selected as copy/move constructors.

		// This is probably a mistake if you expect TFunction to take a copy of what
		// TFunctionRef is bound to, because that's not possible.
		//
		// If you really intended to bind a TFunction to a TFunctionRef, you can just
		// wrap it in a lambda (and thus it's clear you're just binding to a call to another
		// reference):
		//
		// TFunction<int32(float)> MyFunction = [MyFunctionRef](float F) { return MyFunctionRef(F); };
		static_assert(!::TIsTFunctionRef<std::decay_t<FunctorType>>::Value, "Cannot construct a TFunction from a TFunctionRef");
	}

	TFunction(TFunction&&) = default;
	TFunction(const TFunction& Other) = default;
	~TFunction() = default;

	/**
	 * Move assignment operator.
	 */
	TFunction& operator=(TFunction&& Other)
	{
		Swap(*this, Other);
		return *this;
	}

	/**
	 * Copy assignment operator.
	 */
	TFunction& operator=(const TFunction& Other)
	{
		TFunction Temp = Other;
		Swap(*this, Temp);
		return *this;
	}

	/**
	 * Assignment to null is a form of Reset.
	 */
	TFunction& operator=(TYPE_OF_NULLPTR)
	{
		Super::Reset();
		return *this;
	}

	/**
	 * Tests if the TFunction is callable.
	 */
	FORCEINLINE explicit operator bool() const
	{
		return Super::IsSet();
	}

	/**
	 * Tests if the TFunction is callable.
	 */
	FORCEINLINE bool IsSet() const
	{
		return Super::IsSet();
	}

	/**
	 * Nullptr equality operator.
	 */
	FORCEINLINE bool operator==(TYPE_OF_NULLPTR) const
	{
		return !*this;
	}

	/**
	 * Nullptr inequality operator.
	 */
	FORCEINLINE bool operator!=(TYPE_OF_NULLPTR) const
	{
		return (bool)*this;
	}
};

/**
 * TUniqueFunction<FuncType>
 *
 * Used like TFunction above, but is move-only.  This allows non-copyable functors to be bound to it.
 *
 * Example:
 *
 * TUniquePtr<FThing> Thing = MakeUnique<FThing>();
 *
 * TFunction      <void()> CopyableFunc = [Thing = MoveTemp(Thing)](){ Thing->DoSomething(); }; // error - lambda is not copyable
 * TUniqueFunction<void()> MovableFunc  = [Thing = MoveTemp(Thing)](){ Thing->DoSomething(); }; // ok
 *
 * void Foo(TUniqueFunction<void()> Func);
 * Foo(MovableFunc);           // error - TUniqueFunction is not copyable
 * Foo(MoveTemp(MovableFunc)); // ok
 */
template <typename Ret, typename... ParamTypes>
class TUniqueFunction<Ret(ParamTypes...)> final : public UE::Core::Private::Function::TFunctionRefBase<UE::Core::Private::Function::TFunctionStorage<true>, Ret(ParamTypes...)>
{
	using Super = UE::Core::Private::Function::TFunctionRefBase<UE::Core::Private::Function::TFunctionStorage<true>, Ret(ParamTypes...)>;

public:
	/**
	 * Default constructor.
	 */
	TUniqueFunction(TYPE_OF_NULLPTR = nullptr)
	{
	}

	/**
	 * Constructor which binds a TFunction to any function object.
	 */
	template <typename FunctorType>
		requires(!::TIsTUniqueFunction<std::decay_t<FunctorType>>::Value && !::TIsTFunction<std::decay_t<FunctorType>>::Value && std::is_invocable_r_v<Ret, std::decay_t<FunctorType>, ParamTypes...>)
	TUniqueFunction(FunctorType&& InFunc)
		: Super(static_cast<UE::Core::Private::Function::TFuncPtrTypeIfPossible_T<FunctorType>>(InFunc))
	{
		// This constructor is disabled for TUniqueFunction types so it isn't incorrectly selected as copy/move constructors.

		// This is probably a mistake if you expect TUniqueFunction to take a copy of what
		// TFunctionRef is bound to, because that's not possible.
		//
		// If you really intended to bind a TUniqueFunction to a TFunctionRef, you can just
		// wrap it in a lambda (and thus it's clear you're just binding to a call to another
		// reference):
		//
		// TUniqueFunction<int32(float)> MyFunction = [MyFunctionRef](float F) { return MyFunctionRef(F); };
		static_assert(!::TIsTFunctionRef<std::decay_t<FunctorType>>::Value, "Cannot construct a TUniqueFunction from a TFunctionRef");
	}

	/**
	 * Constructor which takes ownership of a TFunction's functor.
	 */
	TUniqueFunction(TFunction<Ret(ParamTypes...)>&& Other)
		: Super(MoveTemp(*(UE::Core::Private::Function::TFunctionRefBase<UE::Core::Private::Function::TFunctionStorage<false>, Ret(ParamTypes...)>*)&Other))
	{
	}

	/**
	 * Constructor which takes ownership of a TFunction's functor.
	 */
	TUniqueFunction(const TFunction<Ret(ParamTypes...)>& Other)
		: Super(*(const UE::Core::Private::Function::TFunctionRefBase<UE::Core::Private::Function::TFunctionStorage<false>, Ret(ParamTypes...)>*)&Other)
	{
	}

	/**
	 * Copy/move assignment operator.
	 */
	TUniqueFunction& operator=(TUniqueFunction&& Other)
	{
		Swap(*this, Other);
		return *this;
	}

	TUniqueFunction(TUniqueFunction&&) = default;
	TUniqueFunction(const TUniqueFunction& Other) = delete;
	TUniqueFunction& operator=(const TUniqueFunction& Other) = delete;
	~TUniqueFunction() = default;

	/**
	 * Removes any bound callable from the TFunction, restoring it to the default 'empty' state.
	 */
	void Reset()
	{
		*this = nullptr;
	}

	/**
	 * Tests if the TUniqueFunction is callable.
	 */
	FORCEINLINE explicit operator bool() const
	{
		return Super::IsSet();
	}

	/**
	 * Tests if the TUniqueFunction is callable.
	 */
	FORCEINLINE bool IsSet() const
	{
		return Super::IsSet();
	}
};


#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
/**
 * Nullptr equality operator.
 */
template <typename FuncType>
FORCEINLINE bool operator==(TYPE_OF_NULLPTR, const TFunction<FuncType>& Func)
{
	return !Func;
}

/**
 * Nullptr inequality operator.
 */
template <typename FuncType>
FORCEINLINE bool operator!=(TYPE_OF_NULLPTR, const TFunction<FuncType>& Func)
{
	return (bool)Func;
}
#endif
