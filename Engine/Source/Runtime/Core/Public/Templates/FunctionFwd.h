// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_exports

/**
 * TFunction<FuncType>
 *
 * See the class definition for intended usage.
 */
template <typename FuncType>
class TFunction;

/**
 * TUniqueFunction<FuncType>
 *
 * See the class definition for intended usage.
 */
template <typename FuncType>
class TUniqueFunction;

/**
 * TFunctionRef<FuncType>
 *
 * See the class definition for intended usage.
 */
template <typename FuncType>
class TFunctionRef;

/**
 * Traits class which checks if T is a TFunction<> type.
 */
template <typename T> struct TIsTFunction               { enum { Value = false }; };
template <typename T> struct TIsTFunction<TFunction<T>> { enum { Value = true  }; };

template <typename T> struct TIsTFunction<const          T> { enum { Value = TIsTFunction<T>::Value }; };
template <typename T> struct TIsTFunction<      volatile T> { enum { Value = TIsTFunction<T>::Value }; };
template <typename T> struct TIsTFunction<const volatile T> { enum { Value = TIsTFunction<T>::Value }; };

/**
 * Traits class which checks if T is a TFunction<> type.
 */
template <typename T> struct TIsTUniqueFunction                     { enum { Value = false }; };
template <typename T> struct TIsTUniqueFunction<TUniqueFunction<T>> { enum { Value = true  }; };

template <typename T> struct TIsTUniqueFunction<const          T> { enum { Value = TIsTUniqueFunction<T>::Value }; };
template <typename T> struct TIsTUniqueFunction<      volatile T> { enum { Value = TIsTUniqueFunction<T>::Value }; };
template <typename T> struct TIsTUniqueFunction<const volatile T> { enum { Value = TIsTUniqueFunction<T>::Value }; };

/**
 * Traits class which checks if T is a TFunctionRef<> type.
 */
template <typename T> struct TIsTFunctionRef                  { enum { Value = false }; };
template <typename T> struct TIsTFunctionRef<TFunctionRef<T>> { enum { Value = true  }; };

template <typename T> struct TIsTFunctionRef<const          T> { enum { Value = TIsTFunctionRef<T>::Value }; };
template <typename T> struct TIsTFunctionRef<      volatile T> { enum { Value = TIsTFunctionRef<T>::Value }; };
template <typename T> struct TIsTFunctionRef<const volatile T> { enum { Value = TIsTFunctionRef<T>::Value }; };

// IWYU pragma: end_exports
