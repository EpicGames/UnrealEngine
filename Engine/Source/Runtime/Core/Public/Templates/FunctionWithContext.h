// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/Invoke.h"
#include <type_traits>

namespace UE
{

/**
 * Type that adapts a callable into a function pointer with a context pointer.
 *
 * This does not take ownership of the callable. If constructed with a lambda,
 * it is only valid until the lambda goes out of scope. Use TFunction to take
 * ownership of the lambda.
 *
 * This behaves like a nullable TFunctionRef with the addition of accessors to
 * pass on the function and context pointers to implementation functions. This
 * tends to generate more efficient code than when passing a TFunctionRef or a
 * TFunctionWithContext either by value or by reference.
 *
 * A function taking a string and float and returning int32 usage might be:
 *
 * void ParseLines(FStringView View, void (*Visitor)(void* Context, FStringView Line), void* Context);
 * inline void ParseLines(FStringView View, TFunctionWithContext<void (FStringView Line)> Visitor)
 * {
 *     ParseLines(View, Visitor.GetFunction(), Visitor.GetContext());
 * }
 *
 * The example ParseLines can be called as:
 *
 * ParseLines(Input, [](FStringView Line) { PrintLine(Line); });
 */
template <typename FunctionType>
class TFunctionWithContext;

template <typename ReturnType, typename... ArgTypes>
class TFunctionWithContext<ReturnType (ArgTypes...)>
{
public:
	using FunctionType = ReturnType (void*, ArgTypes...);

	/** Construct from a lambda or a function pointer. */
	template <typename CallableType>
	inline TFunctionWithContext(CallableType&& Callable UE_LIFETIMEBOUND)
		requires (!std::is_same_v<std::decay_t<CallableType>, TFunctionWithContext>) &&
			std::is_invocable_r_v<ReturnType, std::decay_t<CallableType>, ArgTypes...>
		: Function(&Call<std::remove_pointer_t<std::decay_t<CallableType>>>)
		, Context((void*)&Callable)
	{
	}

	/** Assign from a lambda or a function pointer. */
	template <typename CallableType>
	inline TFunctionWithContext& operator=(CallableType&& Callable UE_LIFETIMEBOUND)
		requires (!std::is_same_v<std::decay_t<CallableType>, TFunctionWithContext>) &&
			std::is_invocable_r_v<ReturnType, std::decay_t<CallableType>, ArgTypes...>
	{
		Function = &Call<std::remove_pointer_t<std::decay_t<CallableType>>>;
		Context = (void*)&Callable;
		return *this;
	}

	/** Construct from a function pointer and context. Function and context may both be null. */
	inline explicit TFunctionWithContext(FunctionType* InFunction, void* InContext)
		: Function(InFunction)
		, Context(InContext)
	{
	}

	/** Construct a null function with null context. */
	inline constexpr TFunctionWithContext(decltype(nullptr))
	{
	}

	/** Returns true if the function is non-null. */
	inline explicit operator bool() const
	{
		return !!Function;
	}

	/** Calls the function with the stored context and provided arguments. Function must be non-null. */
	inline ReturnType operator()(ArgTypes... Args) const
	{
		return Function(Context, (ArgTypes&&)Args...);
	}

	inline FunctionType* GetFunction() const
	{
		return Function;
	}

	inline void* GetContext() const
	{
		return Context;
	}

private:
	template <typename CallableType>
	static ReturnType Call(void* Callable, ArgTypes... Args)
	{
		return Invoke(*(CallableType*)Callable, (ArgTypes&&)Args...);
	}

	FunctionType* Function = nullptr;
	void* Context = nullptr;
};

} // UE
