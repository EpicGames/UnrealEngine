// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/OptionalFwd.h"
#include "Templates/UnrealTemplate.h"
#include <gsl/pointers>

class FArchive;

/** Utility function to turn an `TOptional<gsl::not_null<T>>` back into a nullable T. */
template<typename ObjectType>
[[nodiscard]] inline ObjectType* GetRawPointerOrNull(const TOptional<gsl::not_null<ObjectType*>>& Optional)
{
	return Optional.IsSet() ? static_cast<ObjectType*>(Optional.GetValue()) : nullptr;
}

/** Utility function to turn an `TOptional<gsl::strict_not_null<T>>` back into a nullable T. */
template<typename ObjectType>
[[nodiscard]] inline ObjectType* GetRawPointerOrNull(const TOptional<gsl::strict_not_null<ObjectType*>>& Optional)
{
	return Optional.IsSet() ? static_cast<ObjectType*>(Optional.GetValue()) : nullptr;
}


/** Utility function to serialize a `gsl::not_null<T>. */
template<typename ObjectType>
FArchive& operator<<(FArchive& Ar, gsl::not_null<ObjectType>& NotNull)
{
	Ar << *NotNull;
	return Ar;
}

/** Utility function to serialize a `gsl::strict_not_null<T>`. */
template<typename ObjectType>
FArchive& operator<<(FArchive& Ar, gsl::strict_not_null<ObjectType>& NotNull)
{
	Ar << *NotNull;
	return Ar;
}

/** Utility function to hash a `gsl::not_null<T>`. */
template<typename ObjectType>
[[nodiscard]] inline auto GetTypeHash(const gsl::not_null<ObjectType>& NotNull) -> decltype(GetTypeHash(*NotNull))
{
	return GetTypeHash(*NotNull);
}

/** Utility function to hash a `gsl::strict_not_null<T>`. */
template<typename ObjectType>
[[nodiscard]] inline auto GetTypeHash(const gsl::strict_not_null<ObjectType>& NotNull) -> decltype(GetTypeHash(*NotNull))
{
	return GetTypeHash(*NotNull);
}
