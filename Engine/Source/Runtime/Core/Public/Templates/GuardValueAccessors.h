// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Function.h"

/**
 * Guard around saving/restoring a value.
 * Commonly used to make sure a value is restored
 * even if the code early outs in the future.
 * Usage:
 *  	TGuardValueAccessors GuardSomeBool(UE::SomeGetterFunction, UE::SomeSetterFunction, false); // Saves the state, sets the value, and restores it in dtor.
 */
template <typename AssignedType>
struct TGuardValueAccessors
{
	UE_NONCOPYABLE(TGuardValueAccessors)

	[[nodiscard]] explicit TGuardValueAccessors(TFunctionRef<AssignedType()> Getter, TFunction<void(const AssignedType&)>&& InSetter, const AssignedType& NewValue)
		: Setter(MoveTemp(InSetter))
		, OriginalValue(Getter())
	{
		Setter(NewValue);
	}

	~TGuardValueAccessors()
	{
		Setter(OriginalValue);
	}

	/**
	 * Provides read-only access to the original value of the data being tracked by this struct
	 *
	 * @return	a const reference to the original data value
	 */
	UE_FORCEINLINE_HINT const AssignedType& GetOriginalValue() const
	{
		return OriginalValue;
	}
private:
	TFunction<void(const AssignedType&)> Setter;
	AssignedType OriginalValue;
};
