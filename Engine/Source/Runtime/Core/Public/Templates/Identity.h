// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Returns the same type passed to it.  This is useful in a few cases, but mainly for inhibiting template argument deduction in function arguments, e.g.:
 *
 * template <typename T>
 * void Func1(T Val); // Can be called like Func(123) or Func<int>(123);
 *
 * template <typename T>
 * void Func2(typename TIdentity<T>::Type Val); // Must be called like Func<int>(123)
 *
 * Equivalent to C++20's std::type_identity.
 */
template <typename T>
struct TIdentity
{
	using Type = T;
	using type = T;
};

template <typename T>
using TIdentity_T = typename TIdentity<T>::Type;
