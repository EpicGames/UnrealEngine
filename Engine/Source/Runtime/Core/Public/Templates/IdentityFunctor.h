// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

/**
 * A functor which returns whatever is passed to it.  Mainly used for generic composition.
 */
struct FIdentityFunctor
{
	template <typename T>
	UE_FORCEINLINE_HINT constexpr T&& operator()(T&& Val) const
	{
		return (T&&)Val;
	}
};
