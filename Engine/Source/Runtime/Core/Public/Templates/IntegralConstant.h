// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Defines a value metafunction of the given Value.
 */
template <typename T, T Val>
struct TIntegralConstant
{
	static constexpr T Value = Val;
};
