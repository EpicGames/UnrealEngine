// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Traits/MemberPtrOuter.h"
#include <type_traits>


namespace UE::Core::Private
{
	template <typename OuterType, typename TargetType>
	constexpr auto DereferenceIfNecessary(TargetType&& Target, const volatile OuterType* TargetPtr)
		-> decltype((TargetType&&)Target)
	{
		// If the target is the same as or is derived from the outer type, just return it unchanged.
		return (TargetType&&)Target;
	}

	template <typename OuterType, typename TargetType>
	constexpr auto DereferenceIfNecessary(TargetType&& Target, ...)
		-> decltype(*(TargetType&&)Target)
	{
		// If the target is not related to the outer type, assume it's a (possibly smart) pointer and dereference it.
		return *(TargetType&&)Target;
	}
}


/**
 * Invokes a callable with a set of arguments.  Allows the following:
 *
 * - Calling a functor object given a set of arguments.
 * - Calling a function pointer given a set of arguments.
 * - Calling a member function given a reference to an object and a set of arguments.
 * - Calling a member function given a pointer (including smart pointers) to an object and a set of arguments.
 * - Projecting via a data member pointer given a reference to an object.
 * - Projecting via a data member pointer given a pointer (including smart pointers) to an object.
 *
 * See: http://en.cppreference.com/w/cpp/utility/functional/invoke
 */
template <typename FuncType, typename... ArgTypes>
UE_FORCEINLINE_HINT constexpr auto Invoke(FuncType&& Func, ArgTypes&&... Args)
	-> decltype(((FuncType&&)Func)((ArgTypes&&)Args...))
{
	return ((FuncType&&)Func)((ArgTypes&&)Args...);
}

template <typename ReturnType, typename ObjType, typename TargetType>
constexpr auto Invoke(ReturnType ObjType::*pdm, TargetType&& Target)
	-> decltype(UE::Core::Private::DereferenceIfNecessary<ObjType>((TargetType&&)Target, &Target).*pdm)
{
	return UE::Core::Private::DereferenceIfNecessary<ObjType>((TargetType&&)Target, &Target).*pdm;
}

template <
	typename    PtrMemFunType,
	typename    TargetType,
	typename... ArgTypes,
	typename    ObjType = TMemberPtrOuter_T<PtrMemFunType>
>
constexpr auto Invoke(PtrMemFunType PtrMemFun, TargetType&& Target, ArgTypes&&... Args)
	-> decltype((UE::Core::Private::DereferenceIfNecessary<ObjType>((TargetType&&)Target, &Target).*PtrMemFun)((ArgTypes&&)Args...))
{
	return (UE::Core::Private::DereferenceIfNecessary<ObjType>((TargetType&&)Target, &Target).*PtrMemFun)((ArgTypes&&)Args...);
}


/**
 * Wraps up a named non-member function so that it can easily be passed as a callable.
 * This allows functions with overloads or default arguments to be treated correctly.
 *
 * Example:
 *
 * TArray<FMyType> Array = ...;
 *
 * // Doesn't compile, because you can't take the address of an overloaded function when its type needs to be deduced.
 * Algo::SortBy(Array, &LexToString);
 *
 * // Works as expected
 * Algo::SortBy(Array, UE_PROJECTION(LexToString));
 */
#define UE_PROJECTION(FuncName) \
	[](auto&&... Args) -> decltype(auto) \
	{ \
		return FuncName((decltype(Args)&&)Args...); \
	}

/**
 * Wraps up a named member function so that it can easily be passed as a callable.
 * This allows functions with overloads or default arguments to be treated correctly.
 *
 * Example:
 *
 * TArray<UObject*> Array = ...;
 *
 * // Doesn't compile, because &UObject::GetFullName loses the default argument and passes
 * // FString (UObject::*)(const UObject*) to Algo::SortBy<>(), which is not a valid projection.
 * Algo::SortBy(Array, &UObject::GetFullName);
 *
 * // Works as expected
 * Algo::SortBy(Array, UE_PROJECTION_MEMBER(UObject, GetFullName));
 */
#define UE_PROJECTION_MEMBER(Type, FuncName) \
	[](auto&& Obj, auto&&... Args) -> decltype(auto) \
	{ \
		return UE::Core::Private::DereferenceIfNecessary<Type>((decltype(Obj)&&)Obj, &Obj).FuncName((decltype(Args)&&)Args...); \
	}

namespace UE::Core::Private
{
	template <typename, typename FuncType, typename... ArgTypes>
	struct TInvokeResult_Impl
	{
	};

	template <typename FuncType, typename... ArgTypes>
	struct TInvokeResult_Impl<decltype((void)Invoke(std::declval<FuncType>(), std::declval<ArgTypes>()...)), FuncType, ArgTypes...>
	{
		using Type = decltype(Invoke(std::declval<FuncType>(), std::declval<ArgTypes>()...));
	};
}

/**
 * Trait for the type of the result when invoking a callable with the given argument types.
 * Not defined (and thus usable in SFINAE contexts) when the callable cannot be invoked with the given argument types.
 */
template <typename FuncType, typename... ArgTypes>
struct TInvokeResult : UE::Core::Private::TInvokeResult_Impl<void, FuncType, ArgTypes...>
{
};

template <typename FuncType, typename... ArgTypes>
using TInvokeResult_T = typename TInvokeResult<FuncType, ArgTypes...>::Type;

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Templates/UnrealTemplate.h"
#endif

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Traits/MemberFunctionPtrOuterDeprecated.h"
#endif
