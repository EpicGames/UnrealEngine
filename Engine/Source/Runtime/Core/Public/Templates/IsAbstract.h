// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
* Traits class which tests if a type is abstract.
*/
template <typename T>
struct TIsAbstract
{
	enum { Value = __is_abstract(T) };
};
