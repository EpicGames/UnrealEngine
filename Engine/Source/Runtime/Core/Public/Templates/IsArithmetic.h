// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Traits class which tests if a type is arithmetic.
 */
template <typename T>
struct TIsArithmetic
{ 
	enum { Value = false };
};

template <> struct TIsArithmetic<float>       { enum { Value = true }; };
template <> struct TIsArithmetic<double>      { enum { Value = true }; };
template <> struct TIsArithmetic<long double> { enum { Value = true }; };
template <> struct TIsArithmetic<uint8>       { enum { Value = true }; };
template <> struct TIsArithmetic<uint16>      { enum { Value = true }; };
template <> struct TIsArithmetic<uint32>      { enum { Value = true }; };
template <> struct TIsArithmetic<uint64>      { enum { Value = true }; };
template <> struct TIsArithmetic<int8>        { enum { Value = true }; };
template <> struct TIsArithmetic<int16>       { enum { Value = true }; };
template <> struct TIsArithmetic<int32>       { enum { Value = true }; };
template <> struct TIsArithmetic<int64>       { enum { Value = true }; };
template <> struct TIsArithmetic<long>        { enum { Value = true }; };
template <> struct TIsArithmetic<unsigned long> { enum { Value = true }; };
template <> struct TIsArithmetic<bool>        { enum { Value = true }; };
template <> struct TIsArithmetic<WIDECHAR>    { enum { Value = true }; };
template <> struct TIsArithmetic<ANSICHAR>    { enum { Value = true }; };

template <typename T> struct TIsArithmetic<const          T> { enum { Value = TIsArithmetic<T>::Value }; };
template <typename T> struct TIsArithmetic<      volatile T> { enum { Value = TIsArithmetic<T>::Value }; };
template <typename T> struct TIsArithmetic<const volatile T> { enum { Value = TIsArithmetic<T>::Value }; };
