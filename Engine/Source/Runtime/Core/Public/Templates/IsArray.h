// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Traits class which tests if a type is a C++ array.
 */
template <typename T>           struct TIsArray       { inline static constexpr bool Value = false; };
template <typename T>           struct TIsArray<T[]>  { inline static constexpr bool Value = true;  };
template <typename T, uint32 N> struct TIsArray<T[N]> { inline static constexpr bool Value = true;  };

/**
 * Traits class which tests if a type is a bounded C++ array.
 */
template <typename T>           struct TIsBoundedArray       { inline static constexpr bool Value = false; };
template <typename T, uint32 N> struct TIsBoundedArray<T[N]> { inline static constexpr bool Value = true;  };

/**
 * Traits class which tests if a type is an unbounded C++ array.
 */
template <typename T> struct TIsUnboundedArray      { inline static constexpr bool Value = false; };
template <typename T> struct TIsUnboundedArray<T[]> { inline static constexpr bool Value = true;  };
