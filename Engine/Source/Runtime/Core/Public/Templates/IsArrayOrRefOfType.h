// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"


/**
 * Type trait which returns true if the type T is an array or a reference to an array of ArrType.
 */
template <typename T, typename ArrType>
struct TIsArrayOrRefOfType
{
	enum { Value = false };
};

template <typename ArrType> struct TIsArrayOrRefOfType<               ArrType[], ArrType> { enum { Value = true }; };
template <typename ArrType> struct TIsArrayOrRefOfType<const          ArrType[], ArrType> { enum { Value = true }; };
template <typename ArrType> struct TIsArrayOrRefOfType<      volatile ArrType[], ArrType> { enum { Value = true }; };
template <typename ArrType> struct TIsArrayOrRefOfType<const volatile ArrType[], ArrType> { enum { Value = true }; };

template <typename ArrType, unsigned int N> struct TIsArrayOrRefOfType<               ArrType[N], ArrType> { enum { Value = true }; };
template <typename ArrType, unsigned int N> struct TIsArrayOrRefOfType<const          ArrType[N], ArrType> { enum { Value = true }; };
template <typename ArrType, unsigned int N> struct TIsArrayOrRefOfType<      volatile ArrType[N], ArrType> { enum { Value = true }; };
template <typename ArrType, unsigned int N> struct TIsArrayOrRefOfType<const volatile ArrType[N], ArrType> { enum { Value = true }; };

template <typename ArrType, unsigned int N> struct TIsArrayOrRefOfType<               ArrType(&)[N], ArrType> { enum { Value = true }; };
template <typename ArrType, unsigned int N> struct TIsArrayOrRefOfType<const          ArrType(&)[N], ArrType> { enum { Value = true }; };
template <typename ArrType, unsigned int N> struct TIsArrayOrRefOfType<      volatile ArrType(&)[N], ArrType> { enum { Value = true }; };
template <typename ArrType, unsigned int N> struct TIsArrayOrRefOfType<const volatile ArrType(&)[N], ArrType> { enum { Value = true }; };
