// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"


/**
 * Type trait which returns true if the type T is an array or a reference to an array of a type which matches the predicate.
 */
template <typename T, template <typename> class Predicate>
struct TIsArrayOrRefOfTypeByPredicate
{
	enum { Value = false };
};

template <typename ArrType, template <typename> class Predicate> struct TIsArrayOrRefOfTypeByPredicate<               ArrType[], Predicate> { enum { Value = Predicate<ArrType>::Value }; };
template <typename ArrType, template <typename> class Predicate> struct TIsArrayOrRefOfTypeByPredicate<const          ArrType[], Predicate> { enum { Value = Predicate<ArrType>::Value }; };
template <typename ArrType, template <typename> class Predicate> struct TIsArrayOrRefOfTypeByPredicate<      volatile ArrType[], Predicate> { enum { Value = Predicate<ArrType>::Value }; };
template <typename ArrType, template <typename> class Predicate> struct TIsArrayOrRefOfTypeByPredicate<const volatile ArrType[], Predicate> { enum { Value = Predicate<ArrType>::Value }; };

template <typename ArrType, unsigned int N, template <typename> class Predicate> struct TIsArrayOrRefOfTypeByPredicate<               ArrType[N], Predicate> { enum { Value = Predicate<ArrType>::Value }; };
template <typename ArrType, unsigned int N, template <typename> class Predicate> struct TIsArrayOrRefOfTypeByPredicate<const          ArrType[N], Predicate> { enum { Value = Predicate<ArrType>::Value }; };
template <typename ArrType, unsigned int N, template <typename> class Predicate> struct TIsArrayOrRefOfTypeByPredicate<      volatile ArrType[N], Predicate> { enum { Value = Predicate<ArrType>::Value }; };
template <typename ArrType, unsigned int N, template <typename> class Predicate> struct TIsArrayOrRefOfTypeByPredicate<const volatile ArrType[N], Predicate> { enum { Value = Predicate<ArrType>::Value }; };

template <typename ArrType, unsigned int N, template <typename> class Predicate> struct TIsArrayOrRefOfTypeByPredicate<               ArrType(&)[N], Predicate> { enum { Value = Predicate<ArrType>::Value }; };
template <typename ArrType, unsigned int N, template <typename> class Predicate> struct TIsArrayOrRefOfTypeByPredicate<const          ArrType(&)[N], Predicate> { enum { Value = Predicate<ArrType>::Value }; };
template <typename ArrType, unsigned int N, template <typename> class Predicate> struct TIsArrayOrRefOfTypeByPredicate<      volatile ArrType(&)[N], Predicate> { enum { Value = Predicate<ArrType>::Value }; };
template <typename ArrType, unsigned int N, template <typename> class Predicate> struct TIsArrayOrRefOfTypeByPredicate<const volatile ArrType(&)[N], Predicate> { enum { Value = Predicate<ArrType>::Value }; };
