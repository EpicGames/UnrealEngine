// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Determines if T is a struct/class type
 */
template <typename T>
struct TIsClass
{
private:
	template <typename U> static uint16 Func(int U::*);
	template <typename U> static uint8  Func(...);

public:
	enum { Value = !__is_union(T) && sizeof(Func<T>(0)) - 1 };
};
