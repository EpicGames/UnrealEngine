// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

// HEADER_UNIT_SKIP - Deprecated

//UE_DEPRECATED_HEADER(5.7, "IsConst.h has been deprecated- please include <type_traits> and use std::is_const instead.")

/**
 * Traits class which tests if a type is const.
 */
template <typename T>
struct UE_DEPRECATED(5.7, "TIsConst has been deprecated - please use std::is_const instead.") TIsConst
{
	static constexpr bool Value = false;
};

template <typename T>
struct UE_DEPRECATED(5.7, "TIsConst has been deprecated - please use std::is_const instead.") TIsConst<const T>
{
	static constexpr bool Value = true;
};
