// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Determines if T is constructible from a set of arguments.
 */
template <typename T, typename... Args>
struct TIsConstructible
{
	enum { Value = __is_constructible(T, Args...) };
};
