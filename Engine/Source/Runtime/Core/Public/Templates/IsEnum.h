// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

template <typename T>
struct TIsEnum
{
	enum { Value = __is_enum(T) };
};
