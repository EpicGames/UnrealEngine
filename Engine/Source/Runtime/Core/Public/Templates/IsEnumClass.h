// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/AndOrNot.h"

namespace UE::Core::Private::IsEnumClass
{
	template <typename T>
	struct TIsEnumConvertibleToInt
	{
		static char (&Resolve(int))[2];
		static char Resolve(...);

		enum { Value = sizeof(Resolve(T())) - 1 };
	};
}

/**
 * Traits class which tests if a type is an enum class.
 */
template <typename T>
struct TIsEnumClass
{ 
	enum { Value = ::TAndValue<__is_enum(T), ::TNot<::UE::Core::Private::IsEnumClass::TIsEnumConvertibleToInt<T>>>::Value };
};
