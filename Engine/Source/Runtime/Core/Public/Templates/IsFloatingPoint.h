// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Traits class which tests if a type is floating point.
 */
template <typename T>
struct TIsFloatingPoint
{
	enum { Value = false };
};

template <> struct TIsFloatingPoint<float>       { enum { Value = true }; };
template <> struct TIsFloatingPoint<double>      { enum { Value = true }; };
template <> struct TIsFloatingPoint<long double> { enum { Value = true }; };

template <typename T> struct TIsFloatingPoint<const          T> { enum { Value = TIsFloatingPoint<T>::Value }; };
template <typename T> struct TIsFloatingPoint<      volatile T> { enum { Value = TIsFloatingPoint<T>::Value }; };
template <typename T> struct TIsFloatingPoint<const volatile T> { enum { Value = TIsFloatingPoint<T>::Value }; };
