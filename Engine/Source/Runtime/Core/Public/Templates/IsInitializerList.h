// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <initializer_list>

/**
 * Traits class which tests if a type is an initializer list.
 */
template <typename T>
struct TIsInitializerList
{
	static constexpr bool Value = false;
};

template <typename T>
struct TIsInitializerList<std::initializer_list<T>>
{
	static constexpr bool Value = true;
};

template <typename T> struct TIsInitializerList<const          T> { enum { Value = TIsInitializerList<T>::Value }; };
template <typename T> struct TIsInitializerList<      volatile T> { enum { Value = TIsInitializerList<T>::Value }; };
template <typename T> struct TIsInitializerList<const volatile T> { enum { Value = TIsInitializerList<T>::Value }; };
