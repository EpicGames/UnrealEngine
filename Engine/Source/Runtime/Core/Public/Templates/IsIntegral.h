// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Traits class which tests if a type is integral.
 */
template <typename T>
struct TIsIntegral
{
	enum { Value = false };
};

template <> struct TIsIntegral<         bool>      { enum { Value = true }; };
template <> struct TIsIntegral<         char>      { enum { Value = true }; };
template <> struct TIsIntegral<signed   char>      { enum { Value = true }; };
template <> struct TIsIntegral<unsigned char>      { enum { Value = true }; };
template <> struct TIsIntegral<         char16_t>  { enum { Value = true }; };
template <> struct TIsIntegral<         char32_t>  { enum { Value = true }; };
template <> struct TIsIntegral<         wchar_t>   { enum { Value = true }; };
template <> struct TIsIntegral<         short>     { enum { Value = true }; };
template <> struct TIsIntegral<unsigned short>     { enum { Value = true }; };
template <> struct TIsIntegral<         int>       { enum { Value = true }; };
template <> struct TIsIntegral<unsigned int>       { enum { Value = true }; };
template <> struct TIsIntegral<         long>      { enum { Value = true }; };
template <> struct TIsIntegral<unsigned long>      { enum { Value = true }; };
template <> struct TIsIntegral<         long long> { enum { Value = true }; };
template <> struct TIsIntegral<unsigned long long> { enum { Value = true }; };

template <typename T> struct TIsIntegral<const          T> { enum { Value = TIsIntegral<T>::Value }; };
template <typename T> struct TIsIntegral<      volatile T> { enum { Value = TIsIntegral<T>::Value }; };
template <typename T> struct TIsIntegral<const volatile T> { enum { Value = TIsIntegral<T>::Value }; };
