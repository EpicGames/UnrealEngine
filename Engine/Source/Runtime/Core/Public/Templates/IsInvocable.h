// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Invoke.h"

namespace UE::Core::Private::IsInvocable
{
	template <typename T>
	T&& DeclVal();

	template <typename T>
	struct TVoid
	{
		typedef void Type;
	};

	template <typename, typename CallableType, typename... ArgTypes>
	struct TIsInvocableImpl
	{
		enum { Value = false };
	};

	template <typename CallableType, typename... ArgTypes>
	struct TIsInvocableImpl<typename TVoid<decltype(Invoke(DeclVal<CallableType>(), DeclVal<ArgTypes>()...))>::Type, CallableType, ArgTypes...>
	{
		enum { Value = true };
	};
}

/**
 * Traits class which tests if an instance of CallableType can be invoked with
 * a list of the arguments of the types provided.
 *
 * Examples:
 *     IsInvocable<void()>::Value == true
 *     IsInvocable<void(), FString>::Value == false
 *     IsInvocable<void(FString), FString>::Value == true
 *     IsInvocable<void(FString), const TCHAR*>::Value == true
 *     IsInvocable<void(FString), int32>::Value == false
 *     IsInvocable<void(char, float, bool), int, int, int>::Value == true
 *     IsInvocable<TFunction<void(FString)>, FString>::Value == true
 *     IsInvocable<TFunction<void(FString)>, int32>::Value == false
 */
template <typename CallableType, typename... ArgTypes>
struct TIsInvocable : UE::Core::Private::IsInvocable::TIsInvocableImpl<void, CallableType, ArgTypes...>
{
};
