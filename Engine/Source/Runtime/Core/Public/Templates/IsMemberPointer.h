// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/IsPointer.h"

UE_DEPRECATED_HEADER(5.8, "Templates/IsMemberPointer.h has been deprecated, please use <type_traits> and std::is_member_pointer instead.")

/**
 * Traits class which tests if a type is a pointer to member (data member or member function).
 */
template <typename T>
struct UE_DEPRECATED(5.8, "TIsMemberPointer has been deprecated, please use std::is_member_pointer instead.") TIsMemberPointer
{
	enum { Value = false };
};

template <typename T, typename U> struct UE_DEPRECATED(5.8, "TIsMemberPointer has been deprecated, please use std::is_member_pointer instead.") TIsMemberPointer<T U::*> { enum { Value = true }; };

template <typename T> struct UE_DEPRECATED(5.8, "TIsMemberPointer has been deprecated, please use std::is_member_pointer instead.") TIsMemberPointer<const          T> { enum { Value = TIsPointer<T>::Value }; };
template <typename T> struct UE_DEPRECATED(5.8, "TIsMemberPointer has been deprecated, please use std::is_member_pointer instead.") TIsMemberPointer<      volatile T> { enum { Value = TIsPointer<T>::Value }; };
template <typename T> struct UE_DEPRECATED(5.8, "TIsMemberPointer has been deprecated, please use std::is_member_pointer instead.") TIsMemberPointer<const volatile T> { enum { Value = TIsPointer<T>::Value }; };
