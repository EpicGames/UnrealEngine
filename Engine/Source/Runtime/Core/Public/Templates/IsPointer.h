// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Traits class which tests if a type is a pointer.
 */
template <typename T>
struct TIsPointer
{
	enum { Value = false };
};

template <typename T> struct TIsPointer<T*> { enum { Value = true }; };

template <typename T> struct TIsPointer<const          T> { enum { Value = TIsPointer<T>::Value }; };
template <typename T> struct TIsPointer<      volatile T> { enum { Value = TIsPointer<T>::Value }; };
template <typename T> struct TIsPointer<const volatile T> { enum { Value = TIsPointer<T>::Value }; };
