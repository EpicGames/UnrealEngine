// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
* Traits class which tests if a type is polymorphic (virtual).
*/
template <typename T>
struct TIsPolymorphic
{
	enum { Value = __is_polymorphic(T) };
};
