// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Traits class which tests if a type is a signed integral type.
 */
template <typename T>
struct TIsSigned
{
	enum { Value = false };
};

template <> struct TIsSigned<int8>  { enum { Value = true }; };
template <> struct TIsSigned<int16> { enum { Value = true }; };
template <> struct TIsSigned<int32> { enum { Value = true }; };
template <> struct TIsSigned<int64> { enum { Value = true }; };

template <typename T> struct TIsSigned<const          T> { enum { Value = TIsSigned<T>::Value }; };
template <typename T> struct TIsSigned<      volatile T> { enum { Value = TIsSigned<T>::Value }; };
template <typename T> struct TIsSigned<const volatile T> { enum { Value = TIsSigned<T>::Value }; };
