// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/AndOrNot.h"
#include "Templates/IsTriviallyCopyConstructible.h"
#include "Templates/IsTriviallyCopyAssignable.h"
#include <type_traits>

/**
 * Traits class which tests if a type is trivial.
 */
template <typename T>
struct TIsTrivial
{
	enum { Value = TAndValue<std::is_trivially_destructible_v<T>, TIsTriviallyCopyConstructible<T>, TIsTriviallyCopyAssignable<T>>::Value };
};
