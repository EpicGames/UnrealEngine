// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include <type_traits>

/**
 * Traits class which tests if a type has a trivial copy assignment operator.
 */
template <typename T>
struct TIsTriviallyCopyAssignable
{
	enum { Value = std::is_trivially_copy_assignable_v<T> };
};
