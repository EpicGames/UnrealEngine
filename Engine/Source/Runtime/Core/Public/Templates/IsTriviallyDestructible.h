// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

UE_DEPRECATED_HEADER(5.8, "IsTriviallyDestructible.h has been deprecated, please use std::is_trivially_destructible_v instead.")
