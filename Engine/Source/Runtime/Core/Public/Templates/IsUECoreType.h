// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include <type_traits>

/**
* Traits class which tests if a type is a core variant type (e.g. FVector, which supports FVector3f/FVector3d float/double variants.
* Can be used to determine if the provided type is a core variant type in general:
*  e.g. TIsUECoreVariant<FColor>::Value == false
*		 TIsUECoreVariant<FVector>::Value == true
*  and also to determine if it is a variant type of a  particular component type:
*	e.g TIsUECoreVariant<FVector3d, float>::Value == false
*		TIsUECoreVariant<FVector3d, double>::Value == true
*/
template <typename T, typename = void, typename = void>
struct TIsUECoreVariant
{
	enum { Value = false };
};

template <typename T, typename TC>
struct TIsUECoreVariant<T, TC, typename std::enable_if<TIsUECoreVariant<T>::Value && std::is_same<TC, typename T::FReal>::value, void>::type> 
{
	enum { Value = true };
};

template <typename T, typename TC>
struct TIsUECoreVariant<T, TC, typename std::enable_if<TIsUECoreVariant<T>::Value && std::is_same<TC, typename T::IntType>::value, void>::type>
{
	enum { Value = true };
};

/**
 * Traits class which tests if a type is part of the core types included in CoreMinimal.h.
 */
template <typename T>
struct TIsUECoreType 
{ 
	enum { Value = TIsUECoreVariant<T>::Value };	 // Variant types are automatically core types
};