// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IsEnum.h"
#include <type_traits>

/**
 * Tests if a type is a valid argument to a variadic function, e.g. printf.
 */
template <typename T>
struct TIsValidVariadicFunctionArg
{
private:
	using DecayedT = std::decay_t<T>;

public:
	static constexpr bool Value =
		std::is_enum_v      <DecayedT> ||
		std::is_arithmetic_v<DecayedT> ||
		std::is_pointer_v   <DecayedT> ||
		std::is_same_v      <DecayedT, TYPE_OF_NULLPTR>;
};
