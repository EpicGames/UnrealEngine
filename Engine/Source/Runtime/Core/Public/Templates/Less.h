// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Concepts/SameAs.h"
#include "UnrealTemplate.h"

/**
 * Binary predicate class for sorting elements in order.  Assumes < operator is defined for the template type.
 * Forward declaration exists in ContainersFwd.h.
 *
 * See: http://en.cppreference.com/w/cpp/utility/functional/less
 *
 * TLess should not be specialized - an appropriate operator< should be added.
 */
template <typename T /*= void */>
struct TLess
{
	[[nodiscard]] UE_REWRITE constexpr bool operator()(const T& A, const T& B) const
	{
		if constexpr (requires { { A < B } -> UE::CSameAs<bool>; })
		{
			return A < B;
		}
		else
		{
			static_assert(sizeof(T) == 0, "Trying to use TLess<T> where T doesn't have an appropriate operator< overload. Please add bool operator<(T, T), do not specialize TLess.");
		} 
	} //-V591
};

template <>
struct TLess<void>
{
	template <typename T, typename U>
	[[nodiscard]] UE_REWRITE constexpr bool operator()(T&& A, U&& B) const
	{
		if constexpr (requires { { Forward<T>(A) < Forward<U>(B) } -> UE::CSameAs<bool>; })
		{
			return Forward<T>(A) < Forward<U>(B);
		}
		else
		{
			static_assert(sizeof(T) == 0, "Trying to use TLess<void> with types without an appropriate operator< overload. Please add bool operator<(T, U), do not specialize TLess.");
		} 
	} //-V591
};
