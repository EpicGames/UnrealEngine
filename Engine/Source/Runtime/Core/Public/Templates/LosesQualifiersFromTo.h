// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/CopyQualifiersFromTo.h"
#include <type_traits>

/**
 * Tests if qualifiers are lost between one type and another, e.g.:
 *
 * TLosesQualifiersFromTo_V<const    T1,                T2> == true
 * TLosesQualifiersFromTo_V<volatile T1, const volatile T2> == false
 */
template <typename From, typename To>
struct TLosesQualifiersFromTo
{
	enum { Value = !std::is_same_v<TCopyQualifiersFromTo_T<From, To>, To> };
};

template <typename From, typename To>
inline constexpr bool TLosesQualifiersFromTo_V = TLosesQualifiersFromTo<From, To>::Value;
