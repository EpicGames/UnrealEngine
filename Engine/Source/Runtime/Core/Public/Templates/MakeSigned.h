// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Traits class which gets the signed version of an integer type.
 */
template <typename T>
struct TMakeSigned
{
	static_assert(sizeof(T) == 0, "Unsupported type in TMakeSigned<T>.");
};

template <typename T> struct TMakeSigned<const          T> { using Type = const          typename TMakeSigned<T>::Type; };
template <typename T> struct TMakeSigned<      volatile T> { using Type =       volatile typename TMakeSigned<T>::Type; };
template <typename T> struct TMakeSigned<const volatile T> { using Type = const volatile typename TMakeSigned<T>::Type; };

template <> struct TMakeSigned<int8  > { using Type = int8;  };
template <> struct TMakeSigned<uint8 > { using Type = int8;  };
template <> struct TMakeSigned<int16 > { using Type = int16; };
template <> struct TMakeSigned<uint16> { using Type = int16; };
template <> struct TMakeSigned<int32 > { using Type = int32; };
template <> struct TMakeSigned<uint32> { using Type = int32; };
template <> struct TMakeSigned<int64 > { using Type = int64; };
template <> struct TMakeSigned<uint64> { using Type = int64; };
