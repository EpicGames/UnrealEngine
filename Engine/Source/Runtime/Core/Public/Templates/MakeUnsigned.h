// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Traits class which gets the unsigned version of an integer type.
 */
template <typename T>
struct TMakeUnsigned
{
	static_assert(sizeof(T) == 0, "Unsupported type in TMakeUnsigned<T>.");
};

template <typename T> struct TMakeUnsigned<const          T> { using Type = const          typename TMakeUnsigned<T>::Type; };
template <typename T> struct TMakeUnsigned<      volatile T> { using Type =       volatile typename TMakeUnsigned<T>::Type; };
template <typename T> struct TMakeUnsigned<const volatile T> { using Type = const volatile typename TMakeUnsigned<T>::Type; };

template <> struct TMakeUnsigned<int8  > { using Type = uint8;  };
template <> struct TMakeUnsigned<uint8 > { using Type = uint8;  };
template <> struct TMakeUnsigned<int16 > { using Type = uint16; };
template <> struct TMakeUnsigned<uint16> { using Type = uint16; };
template <> struct TMakeUnsigned<int32 > { using Type = uint32; };
template <> struct TMakeUnsigned<uint32> { using Type = uint32; };
template <> struct TMakeUnsigned<int64 > { using Type = uint64; };
template <> struct TMakeUnsigned<uint64> { using Type = uint64; };
