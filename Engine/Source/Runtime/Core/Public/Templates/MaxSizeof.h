// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/** Finds the maximum sizeof the supplied types */
template <typename...>
struct TMaxSizeof;

template <>
struct TMaxSizeof<>
{
	static constexpr uint32 Value = 0;
};

template <typename T, typename... TRest>
struct TMaxSizeof<T, TRest...>
{
	static const uint32 Value = sizeof(T) > TMaxSizeof<TRest...>::Value ? sizeof(T) : TMaxSizeof<TRest...>::Value;
};