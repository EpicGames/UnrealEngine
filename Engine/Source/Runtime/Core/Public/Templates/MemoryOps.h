// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTypeTraits.h"
#include "Traits/UseBitwiseSwap.h"
#include <new> // IWYU pragma: export
#include <type_traits>


namespace UE::Core::Private::MemoryOps
{
	template <typename DestinationElementType, typename SourceElementType>
	constexpr inline bool TCanBitwiseRelocate_V =
		std::is_same_v<DestinationElementType, SourceElementType> || (
			::TIsBitwiseConstructible<DestinationElementType, SourceElementType>::Value &&
			std::is_trivially_destructible_v<SourceElementType>
		);
}

/**
 * Default constructs a range of items in memory.
 *
 * @param	Elements	The address of the first memory location to construct at.
 * @param	Count		The number of elements to destruct.
 */
template <typename ElementType, typename SizeType>
	requires (sizeof(ElementType) > 0 && !!TIsZeroConstructType<ElementType>::Value) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCEINLINE void DefaultConstructItems(void* Address, SizeType Count)
{
	FMemory::Memset(Address, 0, sizeof(ElementType) * Count);
}
template <typename ElementType, typename SizeType>
	requires (sizeof(ElementType) > 0 && !TIsZeroConstructType<ElementType>::Value) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCENOINLINE void DefaultConstructItems(void* Address, SizeType Count)
{
	ElementType* Element = (ElementType*)Address;
	while (Count)
	{
		::new ((void*)Element) ElementType;
		++Element;
		--Count;
	}
}

/**
 * Destructs a single item in memory.
 *
 * @param	Elements	A pointer to the item to destruct.
 *
 * @note: This function is optimized for values of T, and so will not dynamically dispatch destructor calls if T's destructor is virtual.
 */
template <typename ElementType>
FORCEINLINE constexpr void DestructItem(ElementType* Element)
{
	if constexpr (sizeof(ElementType) == 0)
	{
		// Should never get here, but this construct should improve the error messages we get when we try to call this function with incomplete types
	}
	else if constexpr (!std::is_trivially_destructible_v<ElementType>)
	{
		// We need a typedef here because VC won't compile the destructor call below if ElementType itself has a member called ElementType
		typedef ElementType DestructItemsElementTypeTypedef;

		Element->DestructItemsElementTypeTypedef::~DestructItemsElementTypeTypedef();
	}
}

/**
 * Destructs a range of items in memory.
 *
 * @param	Elements	A pointer to the first item to destruct.
 * @param	Count		The number of elements to destruct.
 *
 * @note: This function is optimized for values of T, and so will not dynamically dispatch destructor calls if T's destructor is virtual.
 */
template <typename ElementType, typename SizeType>
	requires (sizeof(ElementType) > 0 && std::is_trivially_destructible_v<ElementType>) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCEINLINE constexpr void DestructItems(ElementType* Element, SizeType Count)
{
}
template <typename ElementType, typename SizeType>
	requires (sizeof(ElementType) > 0 && !std::is_trivially_destructible_v<ElementType>) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCENOINLINE constexpr void DestructItems(ElementType* Element, SizeType Count)
{
	while (Count)
	{
		// We need a typedef here because VC won't compile the destructor call below if ElementType itself has a member called ElementType
		typedef ElementType DestructItemsElementTypeTypedef;

		Element->DestructItemsElementTypeTypedef::~DestructItemsElementTypeTypedef();
		++Element;
		--Count;
	}
}

/**
 * Constructs a range of items into memory from a set of arguments.  The arguments come from an another array.
 *
 * @param	Dest		The memory location to start copying into.
 * @param	Source		A pointer to the first argument to pass to the constructor.
 * @param	Count		The number of elements to copy.
 */
template <typename DestinationElementType, typename SourceElementType, typename SizeType>
	requires (sizeof(DestinationElementType) > 0 && sizeof(SourceElementType) > 0 && !!::TIsBitwiseConstructible<DestinationElementType, SourceElementType>::Value) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCEINLINE void ConstructItems(void* Dest, const SourceElementType* Source, SizeType Count)
{
	if (Count)
	{
		FMemory::Memcpy(Dest, Source, sizeof(SourceElementType) * Count);
	}
}
template <typename DestinationElementType, typename SourceElementType, typename SizeType>
	requires (sizeof(DestinationElementType) > 0 && sizeof(SourceElementType) > 0 && !::TIsBitwiseConstructible<DestinationElementType, SourceElementType>::Value) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCENOINLINE void ConstructItems(void* Dest, const SourceElementType* Source, SizeType Count)
{
	while (Count)
	{
		::new ((void*)Dest) DestinationElementType(*Source);
		++(DestinationElementType*&)Dest;
		++Source;
		--Count;
	}
}

/**
 * Copy assigns a range of items.
 *
 * @param	Dest		The memory location to start assigning to.
 * @param	Source		A pointer to the first item to assign.
 * @param	Count		The number of elements to assign.
 */
template <typename ElementType, typename SizeType>
	requires (sizeof(ElementType) > 0 && std::is_trivially_copy_assignable_v<ElementType>) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCEINLINE void CopyAssignItems(ElementType* Dest, const ElementType* Source, SizeType Count)
{
	FMemory::Memcpy(Dest, Source, sizeof(ElementType) * Count);
}
template <typename ElementType, typename SizeType>
	requires (sizeof(ElementType) > 0 && !std::is_trivially_copy_assignable_v<ElementType>) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCENOINLINE void CopyAssignItems(ElementType* Dest, const ElementType* Source, SizeType Count)
{
	while (Count)
	{
		*Dest = *Source;
		++Dest;
		++Source;
		--Count;
	}
}

/**
 * Relocates a single items to a new memory location as a new type. This is a so-called 'destructive move' for which
 * there is no single operation in C++ but which can be implemented very efficiently in general.
 *
 * @param	Dest		The memory location to relocate to.
 * @param	Source		A pointer to the first item to relocate.
 */
template <typename DestinationElementType, typename SourceElementType, typename SizeType>
FORCEINLINE void RelocateConstructItem(void* Dest, const SourceElementType* Source)
{
	if constexpr (sizeof(DestinationElementType) == 0 || sizeof(SourceElementType) == 0)
	{
		// Should never get here, but this construct should improve the error messages we get when we try to call this function with incomplete types
	}
	// Do a bitwise relocate if TCanBitwiseRelocate_V says we can, but not if the type involved is known not to be bitwise-swappable
	else if constexpr (UE::Core::Private::MemoryOps::TCanBitwiseRelocate_V<DestinationElementType, SourceElementType> && (!std::is_same_v<DestinationElementType, SourceElementType> || TUseBitwiseSwap<SourceElementType>::Value))
	{
		/* All existing UE containers seem to assume trivial relocatability (i.e. memcpy'able) of their members,
		 * so we're going to assume that this is safe here.  However, it's not generally possible to assume this
		 * in general as objects which contain pointers/references to themselves are not safe to be trivially
		 * relocated.
		 *
		 * However, it is not yet possible to automatically infer this at compile time, so we can't enable
		 * different (i.e. safer) implementations anyway. */

		FMemory::Memmove(Dest, Source, sizeof(SourceElementType));
	}
	else
	{
		// We need a typedef here because VC won't compile the destructor call below if SourceElementType itself has a member called SourceElementType
		typedef SourceElementType RelocateConstructItemsElementTypeTypedef;

		::new ((void*)Dest) DestinationElementType(*Source);
		Source->RelocateConstructItemsElementTypeTypedef::~RelocateConstructItemsElementTypeTypedef();
	}
}

/**
 * Relocates a range of items to a new memory location as a new type. This is a so-called 'destructive move' for which
 * there is no single operation in C++ but which can be implemented very efficiently in general.
 *
 * @param	Dest		The memory location to relocate to.
 * @param	Source		A pointer to the first item to relocate.
 * @param	Count		The number of elements to relocate.
 */
template <typename DestinationElementType, typename SourceElementType, typename SizeType>
	requires (sizeof(DestinationElementType) > 0 && sizeof(SourceElementType) > 0 && UE::Core::Private::MemoryOps::TCanBitwiseRelocate_V<DestinationElementType, SourceElementType>) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCEINLINE void RelocateConstructItems(void* Dest, SourceElementType* Source, SizeType Count)
{
	static_assert(!std::is_const_v<SourceElementType>, "RelocateConstructItems: Source cannot be const");

	/* All existing UE containers seem to assume trivial relocatability (i.e. memcpy'able) of their members,
	 * so we're going to assume that this is safe here.  However, it's not generally possible to assume this
	 * in general as objects which contain pointers/references to themselves are not safe to be trivially
	 * relocated.
	 *
	 * However, it is not yet possible to automatically infer this at compile time, so we can't enable
	 * different (i.e. safer) implementations anyway. */

	FMemory::Memmove(Dest, Source, sizeof(SourceElementType) * Count);
}
template <typename DestinationElementType, typename SourceElementType, typename SizeType>
	requires (sizeof(DestinationElementType) > 0 && sizeof(SourceElementType) > 0 && !UE::Core::Private::MemoryOps::TCanBitwiseRelocate_V<DestinationElementType, SourceElementType>) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCENOINLINE void RelocateConstructItems(void* Dest, SourceElementType* Source, SizeType Count)
{
	static_assert(!std::is_const_v<SourceElementType>, "RelocateConstructItems: Source cannot be const");

	while (Count)
	{
		// We need a typedef here because VC won't compile the destructor call below if SourceElementType itself has a member called SourceElementType
		typedef SourceElementType RelocateConstructItemsElementTypeTypedef;

		::new ((void*)Dest) DestinationElementType((SourceElementType&&)*Source);
		++(DestinationElementType*&)Dest;
		(Source++)->RelocateConstructItemsElementTypeTypedef::~RelocateConstructItemsElementTypeTypedef();
		--Count;
	}
}

/**
 * Move constructs a range of items into memory.
 *
 * @param	Dest		The memory location to start moving into.
 * @param	Source		A pointer to the first item to move from.
 * @param	Count		The number of elements to move.
 */
template <typename ElementType, typename SizeType>
	requires (sizeof(ElementType) > 0 && std::is_trivially_copy_constructible_v<ElementType>) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCEINLINE void MoveConstructItems(void* Dest, const ElementType* Source, SizeType Count)
{
	FMemory::Memmove(Dest, Source, sizeof(ElementType) * Count);
}
template <typename ElementType, typename SizeType>
	requires (sizeof(ElementType) > 0 && !std::is_trivially_copy_constructible_v<ElementType>) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCENOINLINE void MoveConstructItems(void* Dest, const ElementType* Source, SizeType Count)
{
	while (Count)
	{
		::new ((void*)Dest) ElementType((ElementType&&)*Source);
		++(ElementType*&)Dest;
		++Source;
		--Count;
	}
}

/**
 * Move assigns a range of items.
 *
 * @param	Dest		The memory location to start move assigning to.
 * @param	Source		A pointer to the first item to move assign.
 * @param	Count		The number of elements to move assign.
 */
template <typename ElementType, typename SizeType>
	requires (sizeof(ElementType) > 0 && std::is_trivially_copy_assignable_v<ElementType>) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCEINLINE void MoveAssignItems(ElementType* Dest, const ElementType* Source, SizeType Count)
{
	FMemory::Memmove(Dest, Source, sizeof(ElementType) * Count);
}
template <typename ElementType, typename SizeType>
	requires (sizeof(ElementType) > 0 && !std::is_trivially_copy_assignable_v<ElementType>) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCENOINLINE void MoveAssignItems(ElementType* Dest, const ElementType* Source, SizeType Count)
{
	while (Count)
	{
		*Dest = (ElementType&&)*Source;
		++Dest;
		++Source;
		--Count;
	}
}

template <typename ElementType, typename SizeType>
	requires (sizeof(ElementType) > 0 && !!TTypeTraits<ElementType>::IsBytewiseComparable) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCEINLINE bool CompareItems(const ElementType* A, const ElementType* B, SizeType Count)
{
	return !Count || !FMemory::Memcmp(A, B, sizeof(ElementType) * Count);
}
template <typename ElementType, typename SizeType>
	requires (sizeof(ElementType) > 0 && !TTypeTraits<ElementType>::IsBytewiseComparable) // the sizeof here should improve the error messages we get when we try to call this function with incomplete types
FORCENOINLINE bool CompareItems(const ElementType* A, const ElementType* B, SizeType Count)
{
	while (Count)
	{
		if (!(*A == *B))
		{
			return false;
		}

		++A;
		++B;
		--Count;
	}

	return true;
}

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Templates/IsTriviallyCopyAssignable.h"
#include "Templates/IsTriviallyCopyConstructible.h"
#include "Templates/Requires.h"
#endif
