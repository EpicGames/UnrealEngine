// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Identity.h"

/**
 * Utilities for concepts checks.
 *
 * In this case, a successful concept check means that a given C++ expression is well-formed.
 * No guarantees are given about the correctness, behavior or complexity of the runtime behaviour of that expression.
 *
 * Concepts are structs with a rather unusual definition:
 *
 * struct CConcept
 * {
 *     template <[...concept parameters...]>
 *     auto Requires([...placeholder variables...]) -> decltype(
 *         [...expression(s) to test the validity of...]
 *     );
 * };
 *
 * The prefix C is reserved for concepts, and concepts should be directly named as an adjective and not like a predicate, i.e.:
 * CEqualityComparable  - good
 * CIsComparable        - bad
 * CHasEqualsOperator   - bad
 *
 * Concepts can be checked using the TModels_V trait or TModels traits class:
 *
 * TModels_V<Concept, [...arguments...]>
 * TModels<Concept, [...arguments...]>::Value
 *
 * The arguments are forwarded to the template parameters of the concept's Requires() function, which will attempt to
 * compile the expression(s) in the return value, and SFINAE is utilized to test whether that succeeded.
 *
 * The placeholders are simply any variable declarations you need to write your expression(s).
 *
 * Note that 'legal C++' doesn't necessarily mean that the expression will compile when used.  The concept
 * check only tests that the syntax is valid.  Instantiation of function template bodies may still fail.
 * See the CContainerLvalueAddable example below.
 */

namespace UE::Core::Private
{
	template <typename Concept, typename... Ts>
	static char (&ModelsResolve(decltype(&Concept::template Requires<Ts...>)*))[2];

	template <typename Concept, typename... Ts>
	static char (&ModelsResolve(...))[1];
}


/**
 * Trait which does concept checking.
 */
template <typename Concept, typename... Args>
constexpr bool TModels_V = sizeof(UE::Core::Private::ModelsResolve<Concept, Args...>(0)) == 2;

/**
 * Traits class which does concept checking.
 */
template <typename Concept, typename... Args>
struct TModels
{
	static constexpr bool Value = TModels_V<Concept, Args...>;
};

/**
 * Helper function which can be used as an expression in a concept to refine ('inherit') another concept.
 * It should be used as expression-based variant of the TModels_V trait.  If the arguments model
 * the given concept, Refines<Concept, Args...> is a valid expression, otherwise it is not.
 *
 * See the CCopyablePointer example below.
 */
template <typename Concept, typename... Args>
auto Refines() -> int(&)[!!TModels_V<Concept, Args...> * 2 - 1];


/************
 * Examples *
 ************/

/**
 * // Definition of a negatable type.
 * struct CNegatable
 * {
 *     template <typename T>
 *     auto Requires(const T& Val) -> decltype(
 *         -Val
 *     );
 * };
 *
 * static_assert( TModels_V<CNegatable, int >); // ints are negatable
 * static_assert(!TModels_V<CNegatable, int*>); // pointers are not negatable
 */

/**
 * // Definition of an incrementable type.
 * // Pre-increment and post-increment must both be legal.
 * struct CIncrementable
 * {
 *     template <typename T>
 *     auto Requires(T& Val) -> decltype(
 *         ++Val,
 *         Val++
 *     );
 * };
 *
 * static_assert( TModels_V<CIncrementable, int  >); // ints are incrementable
 * static_assert( TModels_V<CIncrementable, int* >); // pointers are incrementable
 * static_assert(!TModels_V<CIncrementable, int[]>); // arrays are not incrementable
 */

/**
 * // Definition of comparability between two types.
 * // Requires both == and != and commutability.
 * struct CEqualityComparable
 * {
 *     template <typename T, typename U>
 *     auto Requires(const T& A, const U& B) -> decltype(
 *         A == B,
 *         B == A,
 *         A != B,
 *         B != A
 *     );
 * };
 *
 * static_assert( TModels_V<CEqualityComparable, FArchive*, FArchiveUObject*>); // base pointers are comparable with derived pointers
 * static_assert(!TModels_V<CEqualityComparable, int*,      float*          >); // unrelated pointers are not comparable
 */

/**
 * // Definition of a type with a nested ElementType and can have lvalues of that type added to it.
 * // Note: this is just an example, not a good description of a container.
 * struct CLvalueAddableContainer
 * {
 *     template <typename ContainerType>
 *     auto Requires(ContainerType& Container, typename ContainerType::ElementType& Element) -> decltype(
 *         Container.Add(Element)
 *     );
 * };
 *
 * static_assert(TModels_V<CLvalueAddableContainer, TArray<TUniquePtr<int>>>); // success, but...
 *
 * TUniquePtr<int> Temp;
 * Array.Add(Temp); // compile error when TArray attempts to copy the TUniquePtr
 *
 * This is because TArray doesn't remove its Add(const ElementType&) overload when the element type
 * is non-copyable - instead we get a compile error inside the template instantiation when we attempt the call.
 */

/**
 * // Definition of a type subtractable from itself and whose result is also (convertible to) itself.
 * struct CGroupSubtractable
 * {
 *     template <typename T>
 *     auto Requires(T& Result, const T& A, const T& B) -> decltype(
 *         Result = A - B
 *     );
 * };
 *
 * static_assert( TModels_V<CGroupSubtractable, int >); // ints form a group under subtraction
 * static_assert(!TModels_V<CGroupSubtractable, int*>); // pointers do not result in another pointer under subtraction and so do not form a group
 */

/**
 * // Definition of a copyable type.
 * struct CCopyable
 * {
 *     template <typename T>
 *     auto Requires(T& Dest, const T& Val) -> decltype(
 *         T(Val),
 *         Dest = Val
 *     );
 * };
 *
 * // Definition of a dereferencable type.
 * struct CDereferencable
 * {
 *     template <typename T>
 *     auto Requires(const T& Val) -> decltype(
 *         *Val
 *     );
 * };
 *
 * // Definition of a copyable pointer-like type which requires copyability and dereferencability (refining existing concepts) and bool-testability (stated directly).
 * // Note: this is just an example, not a great concept.
 * struct CCopyablePointer
 * {
 *     template <typename T>
 *     auto Requires(bool& Result, bool AnyBool, const T& Val) -> decltype(
 *         Refines<CCopyable, T>(),
 *         Refines<CDereferencable, T>(),
 *         Result = AnyBool && Val
 *     );
 * };
 *
 * static_assert( TModels_V<CCopyablePointer, int*           >); // raw pointers model this concept
 * static_assert( TModels_V<CCopyablePointer, TSharedPtr<int>>); // TSharedPtrs model this concept
 * static_assert(!TModels_V<CCopyablePointer, TUniquePtr<int>>); // TUniquePtrs are dereferenceable and bool-testable, but not copyable
 * static_assert(!TModels_V<CCopyablePointer, int            >); // ints are copyable and bool-testable, but not dereferencable
 * static_assert(!TModels_V<CCopyablePointer, FString        >); // FStrings are copyable and dereferencable (to get the const TCHAR*), but not bool-testable
 */
