// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UnrealTemplate.h"

// Special tag for TNoDestroy consteval constructor to avoid confusion with the inner type having
// its own tagged consteval constructor
enum ENoDestroyConstEval { NoDestroyConstEval };

/** 
 * Type to wrap a global variable and prevent its destructor being called. The destructor of this
 * type cannot be trivial, but it is empty and compilers should not register an atexit call for the object. 
 */
template<typename T>
union TNoDestroy
{
    using ElementType = T;

    template<typename... ArgTypes>
    [[nodiscard]] explicit consteval TNoDestroy(ENoDestroyConstEval, ArgTypes&&... InArgs)
        : Value(Forward<ArgTypes>(InArgs)...)
    {
    }

    template<typename... ArgTypes>
    [[nodiscard]] explicit constexpr TNoDestroy(ArgTypes&&... InArgs)
        : Value(Forward<ArgTypes>(InArgs)...)
    {
    }

	UE_NONCOPYABLE(TNoDestroy);
    
    constexpr ~TNoDestroy()
    {
    }

    constexpr ElementType* operator&()
    {
        return &Value;
    }

    constexpr const ElementType* operator&() const
    {
        return &Value;
    }

    constexpr ElementType& operator*()
    {
        return Value;
    }

    constexpr const ElementType& operator*() const
    {
        return Value;
    }

    ElementType* operator->()
    {
        return &Value;
    }

    const ElementType* operator->() const
    {
        return &Value;
    }
private:
    ElementType Value;
};
