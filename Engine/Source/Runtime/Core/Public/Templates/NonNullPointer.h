// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/IntrusiveUnsetOptionalState.h"
#include "Misc/NotNull.h"
#include "Misc/OptionalFwd.h"
#include "Templates/Requires.h"
#include "Templates/UnrealTypeTraits.h"

#include <type_traits>

class FArchive;
enum class EDefaultConstructNonNullPtr 
{ 
	Uninitialized,					// Allocate as specifically uninitialized, will return false for IsInitialized()
	UnsafeDoNotUse = Uninitialized	// Old name for Uninitialized, to be deprecated
};

/**
 * TNonNullPtr is a non-nullable pointer to ObjectType that can be used to track references that should never be null.
 * It will never be null during normal operation, but may be uninitialized during object construction and destruction.
 * It supports intrusive optional state so TOptional<TNonNullPtr<ObjectType>> stores an unset optional pointer as uninitialized.
 * If you need a simple template that checks for null in development with no extra behavior, use TNotNull<ObjectType*> instead.
 */
template<typename ObjectType>
class TNonNullPtr
{
public:

	/**
	 * Start as uninitialized, so it can be used as a member variable and set during later object initialization
	 */
	UE_FORCEINLINE_HINT TNonNullPtr(EDefaultConstructNonNullPtr)
		: Object(nullptr)
	{	
	}

	/**
	 * nullptr constructor - not allowed.
	 */
	UE_FORCEINLINE_HINT TNonNullPtr(TYPE_OF_NULLPTR)
	{
		// Essentially static_assert(false), but this way prevents GCC/Clang from crying wolf by merely inspecting the function body
		static_assert(sizeof(ObjectType) == 0, "Tried to initialize TNonNullPtr with a null pointer!");
	}

	/**
	 * Constructs a non-null pointer from the provided pointer. Must not be nullptr.
	 */
	inline TNonNullPtr(ObjectType* InObject)
		: Object(InObject)
	{
		ensureMsgf(InObject, TEXT("Tried to initialize TNonNullPtr with a null pointer!"));
	}

#if UE_ENABLE_NOTNULL_WRAPPER
	/**
	 * Constructs a non-null pointer from a TNotNull wrapper.
	 */
	template <
		typename OtherObjectType
		UE_REQUIRES(std::is_convertible_v<OtherObjectType, ObjectType*>)
	>
	UE_FORCEINLINE_HINT TNonNullPtr(TNotNull<OtherObjectType> InObject)
		: Object(InObject)
	{
	}
#endif

	/**
	 * Constructs a non-null pointer from another non-null pointer
	 */
	template <
		typename OtherObjectType
		UE_REQUIRES(std::is_convertible_v<OtherObjectType*, ObjectType*>)
	>
	UE_FORCEINLINE_HINT TNonNullPtr(const TNonNullPtr<OtherObjectType>& Other)
		: Object(Other.Get())
	{
	}

	/**
	 * Assignment operator taking a nullptr - not allowed.
	 */
	inline TNonNullPtr& operator=(TYPE_OF_NULLPTR)
	{
		// Essentially static_assert(false), but this way prevents GCC/Clang from crying wolf by merely inspecting the function body
		static_assert(sizeof(ObjectType) == 0, "Tried to assign a null pointer to a TNonNullPtr!");
		return *this;
	}

	/**
	 * Assignment operator taking a pointer
	 */
	inline TNonNullPtr& operator=(ObjectType* InObject)
	{
		ensureMsgf(InObject, TEXT("Tried to assign a null pointer to a TNonNullPtr!"));
		Object = InObject;
		return *this;
	}

#if UE_ENABLE_NOTNULL_WRAPPER
	/**
	 * Assignment operator taking a TNotNull wrapper.
	 */
	template <
		typename OtherObjectType
		UE_REQUIRES(std::is_convertible_v<OtherObjectType, ObjectType*>)
	>
	inline TNonNullPtr& operator=(TNotNull<OtherObjectType> InObject)
	{
		Object = InObject;
		return *this;
	}
#endif

	/**
	 * Assignment operator taking another TNonNullPtr
	 */
	template <
		typename OtherObjectType
		UE_REQUIRES(std::is_convertible_v<OtherObjectType*, ObjectType*>)
	>
	inline TNonNullPtr& operator=(const TNonNullPtr<OtherObjectType>& Other)
	{
		Object = Other.Get();
		return *this;
	}

	/**
	 * Comparison, will also handle default constructed state
	 */
	UE_FORCEINLINE_HINT bool operator==(const TNonNullPtr& Other) const
	{
		return Object == Other.Object;
	}
#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	UE_FORCEINLINE_HINT bool operator!=(const TNonNullPtr& Other) const
	{
		return Object != Other.Object;
	}
#endif

	/**
	 * Comparison with a raw pointer
	 */
	template <
		typename OtherObjectType
		UE_REQUIRES(UE_REQUIRES_EXPR(std::declval<ObjectType*>() == std::declval<OtherObjectType*>()))
	>
	UE_FORCEINLINE_HINT bool operator==(OtherObjectType* Other) const
	{
		return Object == Other;
	}
	template <
		typename OtherObjectType
		UE_REQUIRES(UE_REQUIRES_EXPR(std::declval<OtherObjectType*>() == std::declval<ObjectType*>()))
	>
	UE_FORCEINLINE_HINT friend bool operator==(OtherObjectType* Lhs, const TNonNullPtr& Rhs)
	{
		return Lhs == Rhs.Object;
	}
#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	template <
		typename OtherObjectType
		UE_REQUIRES(UE_REQUIRES_EXPR(std::declval<ObjectType*>() == std::declval<OtherObjectType*>()))
	>
	UE_FORCEINLINE_HINT bool operator!=(OtherObjectType* Other) const
	{
		return Object != Other;
	}
	template <
		typename OtherObjectType
		UE_REQUIRES(UE_REQUIRES_EXPR(std::declval<OtherObjectType*>() == std::declval<ObjectType*>()))
	>
	UE_FORCEINLINE_HINT friend bool operator!=(OtherObjectType* Lhs, const TNonNullPtr& Rhs)
	{
		return Lhs != Rhs.Object;
	}
#endif

	/**
	 * Returns the internal pointer
	 */
	inline operator ObjectType*() const
	{
		ensureMsgf(Object, TEXT("Tried to access null pointer!"));
		return Object;
	}

	/**
	 * Returns the internal pointer
	 */
	inline ObjectType* Get() const
	{
		ensureMsgf(Object, TEXT("Tried to access null pointer!"));
		return Object;
	}

	/**
	 * Dereference operator returns a reference to the object this pointer points to
	 */
	inline ObjectType& operator*() const
	{
		ensureMsgf(Object, TEXT("Tried to access null pointer!"));
		return *Object;
	}

	/**
	 * Arrow operator returns a pointer to this pointer's object
	 */
	inline ObjectType* operator->() const
	{
		ensureMsgf(Object, TEXT("Tried to access null pointer!"));
		return Object;
	}

	/**
	 * Check to see if a pointer has been initialized, which means it will point to a valid object for the lifecycle of the owning object.
	 * Pointers here should always be valid but might be in the EDefaultConstructNonNullPtr state during initialization.
	 */
	UE_FORCEINLINE_HINT bool IsInitialized() const
	{
		return Object != nullptr;
	}

	/**
	 * WARNING: Hack that can be used under extraordinary circumstances. Explicitly reset pointer to uninitialized state
	 * This should only be used during object shutdown or recycling to avoid dangling pointers.
	 */
	inline void ResetToUninitialized()
	{
		Object = nullptr;
	}
	
	/**
	 * Use IsInitialized if needed
	 */
	explicit operator bool() const = delete;

	////////////////////////////////////////////////////
	// Start - intrusive TOptional<TNonNullPtr> state //
	////////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TNonNullPtr;
	UE_FORCEINLINE_HINT explicit TNonNullPtr(FIntrusiveUnsetOptionalState)
		: Object(nullptr)
	{
	}
	UE_FORCEINLINE_HINT bool operator==(FIntrusiveUnsetOptionalState) const
	{
		return Object == nullptr;
	}
	//////////////////////////////////////////////////
	// End - intrusive TOptional<TNonNullPtr> state //
	//////////////////////////////////////////////////

private:
	friend UE_FORCEINLINE_HINT uint32 GetTypeHash(const TNonNullPtr& InPtr)
	{
		return PointerHash(InPtr.Object);
	}

	/** The object we're holding a reference to. */
	ObjectType* Object;
};

/** Convenience function to turn an `TOptional<TNonNullPtr<T>>` back into a nullable T* */
template<typename ObjectType>
inline ObjectType* GetRawPointerOrNull(const TOptional<TNonNullPtr<ObjectType>>& Optional)
{
	return Optional.IsSet() ? Optional->Get() : nullptr;
}
