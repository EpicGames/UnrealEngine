// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Deprecated

#include "CoreTypes.h"

UE_DEPRECATED_HEADER(5.6, "Templates/NotNull.h has been renamed to Templates/GSLNotNull.h to avoid confusion with Misc/NotNull.h which contains the TNotNull type")

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Templates/GSLNotNull.h"
#endif
