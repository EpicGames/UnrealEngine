// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * UE::Overload allows the combining of multiple invocables into a single object where they are overloaded.
 *
 * Example:
 *
 * TTuple<int32, FString, float> Tup1 = ...;
 * TTuple<TArray<int32>, const UObject*, TCHAR> Tup2 = ...;
 * VisitTupleElements(
 *     UE::Overload(
 *         [](int32 Int, const TArray<int32>& Arr)
 *         {
 *             // Called with (Tup1.Get<0>(), Tup2.Get<0>())
 *         },
 *         [Capture1](const FString& Str, const UObject* Obj)
 *         {
 *             // Called with (Tup1.Get<1>(), Tup2.Get<1>())
 *         },
 *         [Capture1, Capture2](float Val, TCHAR Ch)
 *         {
 *             // Called with (Tup1.Get<2>(), Tup2.Get<2>())
 *         },
 *     )
 * )
 *
 * Warning:
 *
 * All of the invocables' captured state is copied into the result.  If the same capture is present
 * in multiple callables, e.g. Capture1 above, the overload object will contain multiple copies of that capture.
 *
 * Another pitfall is to capture-by-move multiple times - only one of the captures will be valid:
 *
 * FString Name = ...;
 * Visit(
 *     UE::Overload(
 *         [Name = MoveTemp(Name)](const FType1& Val1)
 *         {
 *             Val1.DoThing1(Name); // Either this...
 *         },
 *         [Name = MoveTemp(Name)](const FType2& Val2)
 *         {
 *             Val2.DoThing2(Name); // ... or this will be an empty name.
 *         }
 *     ),
 *     Variant
 * );
 *
 * If these are problems, consider writing a bespoke struct with one copy of the capture with multiple
 * operator() overloads.
 */

#include <type_traits>

#include "HAL/Platform.h"
#include "Templates/Projection.h"

namespace UE::Core::Private
{
	template <typename... InvocableTypes>
	struct TOverload : InvocableTypes...
	{
		using InvocableTypes::operator()...;
	};

	template <typename RetType, typename... ArgTypes>
	struct TOverloadWrapper
	{
		RetType (*Callable)(ArgTypes...);

		UE_REWRITE constexpr RetType operator()(ArgTypes... Args) const
		{
			return Callable((ArgTypes&&)Args...);
		}
	};

	// This wraps function pointers in an object so they can be inherited by TOverload
	template <typename RetType, typename... ArgTypes>
	[[nodiscard]] UE_REWRITE constexpr TOverloadWrapper<RetType, ArgTypes...> MakeCallableObject(RetType(*Callable)(ArgTypes...))
	{
		return { Callable };
	}

	template <typename CallableType>
		requires (!std::is_pointer_v<std::decay_t<CallableType>>)
	[[nodiscard]] UE_REWRITE constexpr CallableType&& MakeCallableObject(CallableType&& Callable)
	{
		return (CallableType&&)Callable;
	}
}

namespace UE
{
	// Combines a set of invocable objects into one and overloads them.
	template <typename... InvocableTypes>
	[[nodiscard]] UE_REWRITE constexpr auto Overload(InvocableTypes&&... Invocables)
	{
		return UE::Core::Private::TOverload{ UE::Core::Private::MakeCallableObject(Projection((InvocableTypes&&)Invocables))... };
	}
}
