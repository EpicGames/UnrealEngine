// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "CoreTypes.h"
#include <type_traits>

/**
 * Wraps invocation of a function that can return a value or not. If it doesn't, the function call is wrapped into
 * a function that will invoke the provided function and return the provided default value.
 */
template <typename ReturnType, typename FuncType>
struct TOverrideVoidReturnInvoker
{
	TOverrideVoidReturnInvoker(ReturnType InDefaultValue, FuncType InFunction)
		: DefaultValue(InDefaultValue)
		, Function(InFunction)
	{}

	template <typename... ArgTypes>
	ReturnType operator()(ArgTypes&&... Args)
	{
		using FuncResultType = std::invoke_result_t<FuncType, ArgTypes...>;
		static_assert(std::disjunction_v<std::is_same<FuncResultType, void>, std::is_same<FuncResultType, ReturnType>>, "Passed in function should either return void or ReturnType");

		if constexpr (std::is_same_v<FuncResultType, void>)
		{
			Invoke(Function, Forward<ArgTypes>(Args)...);
			return DefaultValue;
		}
		else
		{
			return Invoke(Function, Forward<ArgTypes>(Args)...);
		}
	}

	ReturnType DefaultValue;
	FuncType Function;
};