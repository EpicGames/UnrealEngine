// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UnrealTemplate.h"

// Single-ownership smart pointer similar to TUniquePtr but with a few differences which make it
// particularly useful for (but not limited to) implementing the pimpl pattern:
//
// https://en.cppreference.com/w/cpp/language/pimpl
//
// Some of the features:
//
// Like TUniquePtr:
// - Unique ownership - no reference counting.
// - Move-only, no copying by default.
// - Has the same static footprint as a pointer.
//
// Like TSharedPtr:
// - The deleter is determined at binding time and type-erased, allowing the object to be deleted without access to the definition of the type.
// - Has additional heap footprint (but smaller than TSharedPtr).
//
// Unlike both:
// - No custom deleter support.
// - No derived->base pointer conversion support (impossible to implement in C++ in a good way with multiple inheritance, and not typically needed for pimpls).
// - The pointed-to object must be created with its Make function - it cannot take ownership of an existing pointer.
// - No array support.
//
// The main benefits of this class which make it useful for pimpls:
// - Has single-ownership semantics.
// - Has the same performance and footprint as a pointer to the object, and minimal overhead on construction and destruction.
// - Can be added as a class member with a forward-declared type without having to worry about the proper definition of constructors and other special member functions.
// - Can support deep copying, including with forward declared types


/**
 * Specifies the copy mode for TPimplPtr
 */
enum class EPimplPtrMode : uint8
{
	/** Don't support copying (default) */
	NoCopy,

	/** Support deep copying, including of forward declared types */
	DeepCopy
};

// Forward declaration
template<typename T, EPimplPtrMode Mode = EPimplPtrMode::NoCopy> struct TPimplPtr;


namespace UE::Core::Private::PimplPtr
{
	inline constexpr SIZE_T RequiredAlignment = 16;

	template <typename T>
	struct TPimplHeapObjectImpl;


	template <typename T>
	void DeleterFunc(void* Ptr)
	{
		// We never pass a null pointer to this function, but the compiler emits delete code
		// which handles nulls - we don't need this extra branching code, so assume it's not.
		UE_ASSUME(Ptr);
		delete (TPimplHeapObjectImpl<T>*)Ptr;
	}

	template<typename T>
	static void* CopyFunc(void* A)
	{
		using FHeapType = TPimplHeapObjectImpl<T>;

		FHeapType* NewHeap = new FHeapType(A);

		return &NewHeap->Val;
	};

	using FDeleteFunc = void(*)(void*);
	using FCopyFunc = void*(*)(void*);

	template <typename T>
	struct TPimplHeapObjectImpl
	{
		enum class ENoCopyType { ConstructType };
		enum class EDeepCopyType { ConstructType };

		template <typename... ArgTypes>
		explicit TPimplHeapObjectImpl(ENoCopyType, ArgTypes&&... Args)
			: Val(Forward<ArgTypes>(Args)...)
		{
			// This should never fire, unless a compiler has laid out this struct in an unexpected way
			static_assert(STRUCT_OFFSET(TPimplHeapObjectImpl, Val) == RequiredAlignment,
							"Unexpected alignment of T within the pimpl object");
		}

		template <typename... ArgTypes>
		explicit TPimplHeapObjectImpl(EDeepCopyType, ArgTypes&&... Args)
			: Copier(&CopyFunc<T>)
			, Val(Forward<ArgTypes>(Args)...)
		{
			// This should never fire, unless a compiler has laid out this struct in an unexpected way
			static_assert(STRUCT_OFFSET(TPimplHeapObjectImpl, Val) == RequiredAlignment,
							"Unexpected alignment of T within the pimpl object");
		}

		explicit TPimplHeapObjectImpl(void* InVal)
			: Copier(&CopyFunc<T>)
			, Val(*(T*)InVal)
		{
		}


		FDeleteFunc Deleter					= &DeleterFunc<T>;
		FCopyFunc Copier					= nullptr;

		alignas(RequiredAlignment) T Val;
	};

	inline void CallDeleter(void* Ptr)
	{
		void* ThunkedPtr = (char*)Ptr - RequiredAlignment;

		// 'noexcept' as part of a function signature is a C++17 feature, but its use here
		// can tidy up the codegen a bit.  As we're likely to build with exceptions disabled
		// anyway, this is not something we need a well-engineered solution for right now,
		// so it's simply left commented out until we can rely on it everywhere.
		(*(void(**)(void*) /*noexcept*/)ThunkedPtr)(ThunkedPtr);
	}

	inline void* CallCopier(void* Ptr)
	{
		void* BasePtr = (char*)Ptr - RequiredAlignment;
		void* ThunkedPtr = (char*)BasePtr + sizeof(FDeleteFunc);

		return (*(FCopyFunc*)ThunkedPtr)(Ptr);
	}
}


template <typename T>
struct TPimplPtr<T, EPimplPtrMode::NoCopy>
{
private:
	template <typename, EPimplPtrMode> friend struct TPimplPtr;

	template <typename U, EPimplPtrMode M, typename... ArgTypes>
	friend TPimplPtr<U, M> MakePimpl(ArgTypes&&... Args);

	explicit TPimplPtr(UE::Core::Private::PimplPtr::TPimplHeapObjectImpl<T>* Impl)
		: Ptr(&Impl->Val)
	{
	}

public:
	TPimplPtr() = default;

	TPimplPtr(TYPE_OF_NULLPTR)
	{
	}

	~TPimplPtr()
	{
		if (Ptr)
		{
			UE::Core::Private::PimplPtr::CallDeleter(this->Ptr);
		}
	}

	TPimplPtr(const TPimplPtr&) = delete;
	TPimplPtr& operator=(const TPimplPtr&) = delete;

	// Movable
	TPimplPtr(TPimplPtr&& Other)
		: Ptr(Other.Ptr)
	{
		Other.Ptr = nullptr;
	}

	TPimplPtr& operator=(TPimplPtr&& Other)
	{
		if (&Other != this)
		{
			T* LocalPtr = this->Ptr;
			this->Ptr = Other.Ptr;
			Other.Ptr = nullptr;
			if (LocalPtr)
			{
				UE::Core::Private::PimplPtr::CallDeleter(LocalPtr);
			}
		}
		return *this;
	}

	TPimplPtr& operator=(TYPE_OF_NULLPTR)
	{
		Reset();
		return *this;
	}

	bool IsValid() const
	{
		return !!this->Ptr;
	}

	explicit operator bool() const
	{
		return !!this->Ptr;
	}

	T* operator->() const
	{
		return this->Ptr;
	}

	T* Get() const
	{
		return this->Ptr;
	}

	T& operator*() const
	{
		return *this->Ptr;
	}

	void Reset()
	{
		if (T* LocalPtr = this->Ptr)
		{
			this->Ptr = nullptr;
			UE::Core::Private::PimplPtr::CallDeleter(LocalPtr);
		}
	}

	UE_FORCEINLINE_HINT bool operator==(TYPE_OF_NULLPTR) { return !IsValid(); }
	UE_FORCEINLINE_HINT bool operator!=(TYPE_OF_NULLPTR) { return  IsValid(); }

private:
	T* Ptr = nullptr;
};

template <typename T>
struct TPimplPtr<T, EPimplPtrMode::DeepCopy> : private TPimplPtr<T, EPimplPtrMode::NoCopy>
{
private:
	using Super = TPimplPtr<T, EPimplPtrMode::NoCopy>;

	template <typename U, EPimplPtrMode M, typename... ArgTypes>
	friend TPimplPtr<U, M> MakePimpl(ArgTypes&&... Args);

	// Super here breaks clang
	using TPimplPtr<T, EPimplPtrMode::NoCopy>::TPimplPtr;

public:
	TPimplPtr() = default;
	~TPimplPtr() = default;

	TPimplPtr(const TPimplPtr& A)
	{
		if (A.IsValid())
		{
			this->Ptr = (T*)UE::Core::Private::PimplPtr::CallCopier(A.Ptr);
		}
	}

	TPimplPtr& operator=(const TPimplPtr& A)
	{
		if (&A != this)
		{
			if (IsValid())
			{
				Reset();
			}

			if (A.IsValid())
			{
				this->Ptr = (T*)UE::Core::Private::PimplPtr::CallCopier(A.Ptr);
			}
		}

		return *this;
	}

	TPimplPtr(TPimplPtr&&) = default;
	TPimplPtr& operator=(TPimplPtr&&) = default;

	TPimplPtr(TYPE_OF_NULLPTR)
	{
	}

	inline TPimplPtr& operator=(TYPE_OF_NULLPTR A)
	{
		Super::operator = (A);
		return *this;
	}

	using Super::IsValid;
	using Super::operator bool;
	using Super::operator ->;
	using Super::Get;
	using Super::operator *;
	using Super::Reset;
};

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
template <typename T, EPimplPtrMode Mode> UE_FORCEINLINE_HINT bool operator==(TYPE_OF_NULLPTR, const TPimplPtr<T, Mode>& Ptr) { return !Ptr.IsValid(); }
template <typename T, EPimplPtrMode Mode> UE_FORCEINLINE_HINT bool operator!=(TYPE_OF_NULLPTR, const TPimplPtr<T, Mode>& Ptr) { return  Ptr.IsValid(); }
#endif

/**
 * Heap-allocates an instance of T with the given arguments and returns it as a TPimplPtr.
 *
 * Usage: TPimplPtr<FMyType> MyPtr = MakePimpl<FMyType>(...arguments...);
 *
 * DeepCopy Usage: TPimplPtr<FMyType, EPimplPtrMode::DeepCopy> MyPtr = MakePimpl<FMyType, EPimplPtrMode::DeepCopy>(...arguments...);
 */
template <typename T, EPimplPtrMode Mode = EPimplPtrMode::NoCopy, typename... ArgTypes>
inline TPimplPtr<T, Mode> MakePimpl(ArgTypes&&... Args)
{
	using FHeapType = UE::Core::Private::PimplPtr::TPimplHeapObjectImpl<T>;
	using FHeapConstructType = typename std::conditional<Mode == EPimplPtrMode::NoCopy, typename FHeapType::ENoCopyType,
															typename FHeapType::EDeepCopyType>::type;

	static_assert(Mode != EPimplPtrMode::DeepCopy ||
					std::is_copy_constructible<T>::value, "T must be a copyable type, to use with EPimplPtrMode::DeepCopy");
	static_assert(sizeof(T) > 0, "T must be a complete type");
	static_assert(alignof(T) <= UE::Core::Private::PimplPtr::RequiredAlignment, "T cannot be aligned more than 16 bytes");

	return TPimplPtr<T, Mode>(new FHeapType(FHeapConstructType::ConstructType, Forward<ArgTypes>(Args)...));
}


