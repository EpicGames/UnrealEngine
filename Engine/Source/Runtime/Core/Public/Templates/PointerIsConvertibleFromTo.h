// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/StaticAssertCompleteType.h"
#include "Templates/LosesQualifiersFromTo.h"
#include <type_traits>

namespace UE::Core::Private::PointerIsConvertibleFromTo
{
	template <typename From, typename To, typename NoCVFrom = std::remove_cv_t<From>, typename NoCVTo = std::remove_cv_t<To>>
	struct TImpl
	{
	private:
		static uint8  Test(...);
		static uint16 Test(To*);

		UE_STATIC_ASSERT_COMPLETE_TYPE(From, "TPointerIsConvertibleFromTo must not be instantiated with incomplete types");
		UE_STATIC_ASSERT_COMPLETE_TYPE(To,   "TPointerIsConvertibleFromTo must not be instantiated with incomplete types");

	public:
		enum { Value = sizeof(Test((From*)nullptr)) - 1 };
	};

	template <typename From, typename To, typename NoCVFrom>
	struct TImpl<From, To, NoCVFrom, NoCVFrom>
	{
		// cv T* to cv T* conversions are always allowed as long as no CVs are lost
		enum { Value = !TLosesQualifiersFromTo_V<From, To> };
	};

	template <typename From, typename To, typename NoCVFrom>
	struct TImpl<From, To, NoCVFrom, void>
	{
		// cv T* to cv void* conversions are always allowed as long as no CVs are lost
		enum { Value = !TLosesQualifiersFromTo_V<From, To> };
	};

	template <typename From, typename To>
	struct TImpl<From, To, void, void>
	{
		// cv void* to cv void* conversions are always allowed as long as no CVs are lost
		enum { Value = !TLosesQualifiersFromTo_V<From, To> };
	};

	template <typename From, typename To, typename NoCVTo>
	struct TImpl<From, To, void, NoCVTo>
	{
		// cv void* to cv not_void* conversions are never legal
		enum { Value = false };
	};
}

/**
 * Tests if a From* is convertible to a To*
 **/
template <typename From, typename To>
struct TPointerIsConvertibleFromTo : UE::Core::Private::PointerIsConvertibleFromTo::TImpl<From, To>
{
};


class TPointerIsConvertibleFromTo_TestBase
{
};

class TPointerIsConvertibleFromTo_TestDerived : public TPointerIsConvertibleFromTo_TestBase
{
};

class TPointerIsConvertibleFromTo_Unrelated
{
};

static_assert(TPointerIsConvertibleFromTo<bool, bool>::Value, "Platform TPointerIsConvertibleFromTo test failed.");
static_assert(TPointerIsConvertibleFromTo<void, void>::Value, "Platform TPointerIsConvertibleFromTo test failed.");
static_assert(TPointerIsConvertibleFromTo<bool, void>::Value, "Platform TPointerIsConvertibleFromTo test failed.");
static_assert(TPointerIsConvertibleFromTo<const bool, const void>::Value, "Platform TPointerIsConvertibleFromTo test failed.");
static_assert(TPointerIsConvertibleFromTo<TPointerIsConvertibleFromTo_TestDerived, TPointerIsConvertibleFromTo_TestBase>::Value, "Platform TPointerIsConvertibleFromTo test failed.");
static_assert(TPointerIsConvertibleFromTo<TPointerIsConvertibleFromTo_TestDerived, const TPointerIsConvertibleFromTo_TestBase>::Value, "Platform TPointerIsConvertibleFromTo test failed.");
static_assert(TPointerIsConvertibleFromTo<const TPointerIsConvertibleFromTo_TestDerived, const TPointerIsConvertibleFromTo_TestBase>::Value, "Platform TPointerIsConvertibleFromTo test failed.");
static_assert(TPointerIsConvertibleFromTo<TPointerIsConvertibleFromTo_TestBase, TPointerIsConvertibleFromTo_TestBase>::Value, "Platform TPointerIsConvertibleFromTo test failed.");
static_assert(TPointerIsConvertibleFromTo<TPointerIsConvertibleFromTo_TestBase, void>::Value, "Platform TPointerIsConvertibleFromTo test failed.");

static_assert(!TPointerIsConvertibleFromTo<TPointerIsConvertibleFromTo_TestBase, TPointerIsConvertibleFromTo_TestDerived>::Value, "Platform TPointerIsConvertibleFromTo test failed.");
static_assert(!TPointerIsConvertibleFromTo<TPointerIsConvertibleFromTo_Unrelated, TPointerIsConvertibleFromTo_TestBase>::Value, "Platform TPointerIsConvertibleFromTo test failed.");
static_assert(!TPointerIsConvertibleFromTo<bool, TPointerIsConvertibleFromTo_TestBase>::Value, "Platform TPointerIsConvertibleFromTo test failed.");
static_assert(!TPointerIsConvertibleFromTo<void, TPointerIsConvertibleFromTo_TestBase>::Value, "Platform TPointerIsConvertibleFromTo test failed.");
static_assert(!TPointerIsConvertibleFromTo<TPointerIsConvertibleFromTo_TestBase, bool>::Value, "Platform TPointerIsConvertibleFromTo test failed.");
static_assert(!TPointerIsConvertibleFromTo<void, bool>::Value, "Platform TPointerIsConvertibleFromTo test failed.");
