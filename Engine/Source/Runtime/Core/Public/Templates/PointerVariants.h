// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/PointerConvertibleTo.h"
#include "Concepts/UObject.h"
#include "Misc/TVariant.h"
#include "Templates/Requires.h"
#include "Templates/SharedPointer.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UnrealTemplate.h"

#include <type_traits>

/*
* `TWeakPtrVariant` and `TStrongPtrVariant` are particularly useful for "interfaces" in areas where
* UObjects and non-UObjects exist to hold onto a "smart" pointer to the Interface, rather than needing to keep track of the implementing class to check validity and manually cast
*/

/* `TWeakPtrVariant` will be either a `TWeakObjectPtr` or `TWeakPtr` */
template<typename BaseType>
struct TWeakPtrVariant;

/* `TStrongPtrVariant` will be either a `TStrongObjectPtr` or `TSharedPtr` */
template<typename BaseType>
struct TStrongPtrVariant;

namespace UE::Core::Private
{
	template<typename BaseType, bool bIsStrong>
	struct TPtrVariantBase;

	template <typename LhsType, bool bLhsStrength, typename RhsType, bool bRhsStrength
		UE_REQUIRES(UE_REQUIRES_EXPR((LhsType*)nullptr == (RhsType*)nullptr))
	>
	bool operator==(const TPtrVariantBase<LhsType, bLhsStrength>& Lhs, const TPtrVariantBase<RhsType, bRhsStrength>& Rhs);

	template<typename LhsType, typename RhsType, bool bRhsStrength
		UE_REQUIRES(UE_REQUIRES_EXPR((LhsType*)nullptr == (RhsType*)nullptr))
	>
	bool operator==(LhsType* Lhs, const TPtrVariantBase<RhsType, bRhsStrength>& Rhs);

	template<typename LhsType, bool bLhsStrength, typename RhsType
		UE_REQUIRES(UE_REQUIRES_EXPR((LhsType*)nullptr == (RhsType*)nullptr))
	>
	bool operator==(const TPtrVariantBase<LhsType, bLhsStrength>& Lhs, RhsType* Rhs);

	template<typename BaseType, bool bIsStrong>
	struct TPtrVariantBase
	{
		friend struct TWeakPtrVariant<BaseType>;
		friend struct TStrongPtrVariant<BaseType>;

		using SharedType = std::conditional_t<bIsStrong, TSharedPtr<BaseType>, TWeakPtr<BaseType>>;

		struct FObjectPtrWrapper
		{
			using ObjectPtrType = std::conditional_t<bIsStrong, TStrongObjectPtr<const UObject>, TWeakObjectPtr<const UObject>>;

			FObjectPtrWrapper(const UObject* InObjectPtr, BaseType* const InCastedPtr)
				: ObjectPtr(InObjectPtr)
				, CastedPtr(InCastedPtr)
			{
			}

			ObjectPtrType ObjectPtr;
			BaseType* CastedPtr = nullptr;
		};

	public:
		constexpr TPtrVariantBase() = default;
		constexpr TPtrVariantBase(TYPE_OF_NULLPTR)
		{
		}

		/** Constructs from a raw pointer, either UObject or shareable-from-this. */
		template<CPointerConvertibleTo<BaseType> DerivedType>
		requires (CUObject<DerivedType> || IsDerivedFromSharedFromThis<DerivedType>())
		constexpr TPtrVariantBase(DerivedType* InDerived)
		{
			if constexpr (CUObject<DerivedType>)
			{
				PtrVariant.template Emplace<FObjectPtrWrapper>(Cast<const UObject>(InDerived), Cast<BaseType>(InDerived));
			}
			else
			{
				if constexpr (bIsStrong)
				{
					PtrVariant.template Emplace<SharedType>(StaticCastSharedRef<BaseType>(StaticCastSharedRef<DerivedType>(InDerived->AsShared())).ToSharedPtr());
				}
				else
				{
					PtrVariant.template Emplace<SharedType>(StaticCastSharedRef<BaseType>(StaticCastSharedRef<DerivedType>(InDerived->AsShared())).ToWeakPtr());
				}
			}
		}

		UE_REWRITE constexpr TPtrVariantBase(SharedType&& InSharedPtr)
			: PtrVariant(TInPlaceType<SharedType>(), MoveTemp(InSharedPtr))
		{
		}

		TPtrVariantBase& operator=(TYPE_OF_NULLPTR)
		{
			Reset();
			return *this;
		}

		bool IsValid() const
		{
			return ::Visit([](auto& StoredValue)
				{
					using StoredValueType = std::decay_t<decltype(StoredValue)>;
					if constexpr (std::is_same_v<StoredValueType, SharedType>)
					{
						return StoredValue.IsValid();
					}
					else
					{
						return StoredValue.ObjectPtr.IsValid();
					}
				}, PtrVariant);
		}

		void Reset()
		{
			::Visit([](auto& StoredValue)
				{
					using StoredValueType = std::decay_t<decltype(StoredValue)>;
					if constexpr (std::is_same_v<StoredValueType, SharedType>)
					{
						StoredValue.Reset();
					}
					else
					{
						StoredValue.ObjectPtr.Reset();
					}
				}, PtrVariant);
		}

		/** Returns true if this variant holds a UObject pointer (as opposed to a TSharedPtr). */
		bool IsUObject() const
		{
			return PtrVariant.template IsType<FObjectPtrWrapper>();
		}

	private:
		template <typename LhsType, bool bLhsStrength, typename RhsType, bool bRhsStrength
			UE_REQUIRES_FRIEND(UE_REQUIRES_EXPR((LhsType*)nullptr == (RhsType*)nullptr))
		>
		friend bool operator==(const TPtrVariantBase<LhsType, bLhsStrength>& Lhs, const TPtrVariantBase<RhsType, bRhsStrength>& Rhs);

		template<typename LhsType, typename RhsType, bool bRhsStrength
			UE_REQUIRES_FRIEND(UE_REQUIRES_EXPR((LhsType*)nullptr == (RhsType*)nullptr))
		>
		friend bool operator==(LhsType* Lhs, const TPtrVariantBase<RhsType, bRhsStrength>& Rhs);

		template<typename LhsType, bool bLhsStrength, typename RhsType
			UE_REQUIRES_FRIEND(UE_REQUIRES_EXPR((LhsType*)nullptr == (RhsType*)nullptr))
		>
		friend bool operator==(const TPtrVariantBase<LhsType, bLhsStrength>& Lhs, RhsType* Rhs);

		const BaseType* GetRawPtrValue_Internal() const
		{
			return ::Visit([this](auto& StoredValue)
					{
						using StoredValueType = std::decay_t<decltype(StoredValue)>;
						if constexpr (std::is_same_v<StoredValueType, SharedType>)
						{
							if constexpr (bIsStrong)
							{
								return IsValid() ? StoredValue.Get() : (const BaseType*)nullptr;
							}
							else
							{
								return IsValid() ? StoredValue.Pin().Get() : (const BaseType*)nullptr;
							}
						}
						else
						{
							return IsValid() ? StoredValue.CastedPtr : (const BaseType*)nullptr;
						}
					}, PtrVariant);
		}

		TVariant<SharedType, FObjectPtrWrapper> PtrVariant;
	};

	template <typename LhsType, bool bLhsStrength, typename RhsType, bool bRhsStrength
		UE_REQUIRES_DEFINITION(UE_REQUIRES_EXPR((LhsType*)nullptr == (RhsType*)nullptr))
	>
	bool operator==(const TPtrVariantBase<LhsType, bLhsStrength>& Lhs, const TPtrVariantBase<RhsType, bRhsStrength>& Rhs)
	{
		return Lhs.GetRawPtrValue_Internal() == Rhs.GetRawPtrValue_Internal();
	}

	template <typename LhsType, bool bLhsStrength, typename RhsType, bool bRhsStrength>
	bool operator!=(const TPtrVariantBase<LhsType, bLhsStrength>& Lhs, const TPtrVariantBase<RhsType, bRhsStrength>& Rhs)
	{
		return !(Lhs == Rhs);
	}

	template<typename LhsType, typename RhsType, bool bRhsStrength
		UE_REQUIRES_DEFINITION(UE_REQUIRES_EXPR((LhsType*)nullptr == (RhsType*)nullptr))
	>
	bool operator==(LhsType* Lhs, const TPtrVariantBase<RhsType, bRhsStrength>& Rhs)
	{
		return Lhs == Rhs.GetRawPtrValue_Internal();
	}

	template<typename LhsType, typename RhsType, bool bRhsStrength>
	bool operator!=(LhsType* Lhs, const TPtrVariantBase<RhsType, bRhsStrength>& Rhs)
	{
		return !(Lhs == Rhs);
	}

	template<typename LhsType, bool bLhsStrength, typename RhsType
		UE_REQUIRES_DEFINITION(UE_REQUIRES_EXPR((LhsType*)nullptr == (RhsType*)nullptr))
	>
	bool operator==(const TPtrVariantBase<LhsType, bLhsStrength>& Lhs, RhsType* Rhs)
	{
		return Lhs.GetRawPtrValue_Internal() == Rhs;
	}

	template<typename LhsType, bool bLhsStrength, typename RhsType>
	bool operator!=(const TPtrVariantBase<LhsType, bLhsStrength>& Lhs, RhsType* Rhs)
	{
		return !(Lhs == Rhs);
	}
} // namespace UE::Core::Private

template<typename BaseType>
struct TStrongPtrVariant : public UE::Core::Private::TPtrVariantBase<BaseType, true>
{
	using Super = UE::Core::Private::TPtrVariantBase<BaseType, true>;
	using Super::Super;

	template<UE::CPointerConvertibleTo<BaseType> DerivedType>
	UE_REWRITE constexpr TStrongPtrVariant(TSharedPtr<DerivedType>&& SharedPtr)
		: Super(MoveTemp(SharedPtr))
	{
	}

	template<UE::CPointerConvertibleTo<BaseType> DerivedType>
	UE_REWRITE constexpr TStrongPtrVariant(const TSharedPtr<DerivedType>& SharedPtr)
		: Super(CopyTemp(SharedPtr))
	{
	}

	template<UE::CPointerConvertibleTo<BaseType> DerivedType>
	UE_REWRITE constexpr TStrongPtrVariant(const TSharedRef<DerivedType>& SharedPtr)
		: Super(SharedPtr.ToSharedPtr())
	{
	}

	BaseType* Get() const
	{
		return ::Visit([](const auto& StoredValue)
			{
				using StoredValueType = std::decay_t<decltype(StoredValue)>;
				if constexpr (TIsTSharedPtr_V<StoredValueType>)
				{
					return StoredValue.Get();
				}
				else
				{
					return StoredValue.CastedPtr;
				}
			}, Super::PtrVariant);
	}

	UE_REWRITE BaseType& operator*() const
	{
		return *Get();
	}

	TWeakPtrVariant<BaseType> ToWeakVariant() const
	{
		if (Super::IsValid())
		{
			return ::Visit([](auto& StoredValue)
				{
					TWeakPtrVariant<BaseType> WeakPtrVariant;

					using StoredValueType = std::decay_t<decltype(StoredValue)>;
					if constexpr (TIsTSharedPtr_V<StoredValueType>)
					{
						WeakPtrVariant.PtrVariant.template Emplace<typename TWeakPtrVariant<BaseType>::SharedType>(StoredValue.ToWeakPtr());
					}
					else
					{
						WeakPtrVariant.PtrVariant.template Emplace<typename TWeakPtrVariant<BaseType>::FObjectPtrWrapper>(StoredValue.ObjectPtr.Get(), StoredValue.CastedPtr);
					}

					return WeakPtrVariant;
				}, Super::PtrVariant);
		}
		else
		{
			return TWeakPtrVariant<BaseType>();
		}
	}
};

template<typename BaseType>
struct TWeakPtrVariant : public UE::Core::Private::TPtrVariantBase<BaseType, false>
{
	using Super = UE::Core::Private::TPtrVariantBase<BaseType, false>;
	using Super::Super;

	template<UE::CPointerConvertibleTo<BaseType> DerivedType>
	UE_REWRITE constexpr TWeakPtrVariant(TWeakPtr<DerivedType>&& SharedPtr)
		: Super(MoveTemp(SharedPtr))
	{
	}

	template<UE::CPointerConvertibleTo<BaseType> DerivedType>
	UE_REWRITE constexpr TWeakPtrVariant(const TWeakPtr<DerivedType>& SharedPtr)
		: Super(CopyTemp(SharedPtr))
	{
	}

	TStrongPtrVariant<BaseType> Pin() const
	{
		if (Super::IsValid())
		{
			return ::Visit([](auto& StoredValue)
				{
					TStrongPtrVariant<BaseType> StrongPtrVariant;

					using StoredValueType = std::decay_t<decltype(StoredValue)>;
					if constexpr (TIsTWeakPtr_V<StoredValueType>)
					{
						StrongPtrVariant.PtrVariant.template Emplace<typename TStrongPtrVariant<BaseType>::SharedType>(StoredValue.Pin());
					}
					else
					{
						StrongPtrVariant.PtrVariant.template Emplace<typename TStrongPtrVariant<BaseType>::FObjectPtrWrapper>(StoredValue.ObjectPtr.Get(), StoredValue.CastedPtr);
					}

					return StrongPtrVariant;
				}, Super::PtrVariant);
		}
		else
		{
			return TStrongPtrVariant<BaseType>();
		}
	}
};
