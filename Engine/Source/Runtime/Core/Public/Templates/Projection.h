// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE::Core::Private
{
	template <typename InvocableType>
	struct TProjectionMemberFunction;

	template <typename ClassType, typename FunctionType>
	struct TProjectionMemberFunction<FunctionType ClassType::*>
	{
		FunctionType ClassType::* MemberFunctionPtr;

		template <typename Arg0Type, typename... ArgTypes>
		constexpr decltype(auto) operator()(Arg0Type&& Arg0, ArgTypes&&... Args) const
		{
			using DecayedArg0Type = std::decay_t<Arg0Type>;
			if constexpr (std::is_base_of_v<ClassType, DecayedArg0Type>)
			{
				return (((Arg0Type&&)Arg0).*this->MemberFunctionPtr)((ArgTypes&&)Args...);
			}
			else
			{
				return ((*(Arg0Type&&)Arg0).*this->MemberFunctionPtr)((ArgTypes&&)Args...);
			}
		}
	};

	template <typename InvocableType>
	struct TProjectionMemberData;

	template <typename ClassType, typename MemberType>
	struct TProjectionMemberData<MemberType ClassType::*>
	{
		MemberType ClassType::* DataMemberPtr;

		template <typename Arg0Type>
		constexpr decltype(auto) operator()(Arg0Type&& Arg0) const
		{
			using DecayedArg0Type = std::decay_t<Arg0Type>;
			if constexpr (std::is_base_of_v<ClassType, DecayedArg0Type>)
			{
				return ((Arg0Type&&)Arg0).*this->DataMemberPtr;
			}
			else
			{
				return (*(Arg0Type&&)Arg0).*this->DataMemberPtr;
			}
		}
	};

	template <typename Class, typename MemberType>
	inline constexpr bool TIsMemberPointerToFunction(MemberType Class::*)
	{
		return std::is_function_v<MemberType>;
	}
}

/**
 * Projection() is a related function to Invoke(), in that it can be used to invoke an object with a set of arguments.
 * However, it works by transforming something invocable into something callable, i.e. can be called with a normal parenthesized set of arguments.
 *
 * // These are equivalent:
 * Projection(I)(Args...)
 * Invoke(I, Args...).
 *
 * ... with the same advantages:
 *
 * // Can accept member function pointers
 * Invoke(&FObjType::Member, Obj);            // equivalent to Obj.Member
 * Invoke(&FObjType::Func, Obj, Args...);     // equivalent to Obj.Func(Args...)
 * Projection(&FObjType::Member)(Obj);        // equivalent to Obj.Member
 * Projection(&FObjType::Func)(Obj, Args...); // equivalent to Obj.Func(Args...)
 *
 * // Can operate on pointers, including smart pointers
 * Invoke(&FObjType::Member, ObjPtr);            // equivalent to ObjPtr->Member
 * Invoke(&FObjType::Func, ObjPtr, Args...);     // equivalent to ObjPtr->Func(Args...)
 * Projection(&FObjType::Member)(ObjPtr);        // equivalent to ObjPtr->Member
 * Projection(&FObjType::Func)(ObjPtr, Args...); // equivalent to ObjPtr->Func(Args...)
 *
 * However, Projection() has some additional advantages:
 *
 * - Projection(I) returns I unchanged if it is already callable, meaning no redundant stepping in and out of many Invoke() calls in the debugger.
 * - Projection(...) is variadic and can transform a sequence of invocables into a callable that invokes them one by one:
 *
 * Projection(A, B, C)(Args...)
 * Invoke(C, Invoke(B, Invoke(A, Args...)))
 *
 * This allows users to pass a sequence of projections to an algorithm that takes a single projection:
 *
 * struct FInner
 * {
 *     FString Name;
 * };
 * struct FOuter
 * {
 *     FInner Inner;
 * };
 *
 * // Sort array of outers by the names of inner,
 * Algo::SortBy(ArrayOfOuters, Projection(&FOuter::Inner, &FInner::Name));
 */
template <typename Invocable0Type, typename... InvocableTypes>
[[nodiscard]] constexpr auto Projection(Invocable0Type&& Invocable0, InvocableTypes&&... Invocables)
{
	if constexpr (sizeof...(InvocableTypes) == 0)
	{
		using DecayedInvocable0Type = std::decay_t<Invocable0Type>;
		if constexpr (!std::is_member_pointer_v<DecayedInvocable0Type>)
		{
			return (Invocable0Type&&)Invocable0;
		}
		else if constexpr (UE::Core::Private::TIsMemberPointerToFunction(DecayedInvocable0Type{}))
		{
			return UE::Core::Private::TProjectionMemberFunction<DecayedInvocable0Type>{ (Invocable0Type&&)Invocable0 };
		}
		else
		{
			return UE::Core::Private::TProjectionMemberData<DecayedInvocable0Type>{ (Invocable0Type&&)Invocable0 };
		}
	}
	else
	{
		return [Callable0 = Projection((Invocable0Type&&)Invocable0), CallableRest = Projection((InvocableTypes&&)Invocables...)](auto&&... Args) -> decltype(auto)
		{
			return CallableRest(Callable0((decltype(Args)&&)Args...));
		};
	}
}
