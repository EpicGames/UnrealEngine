// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Async/AtomicRefCount.h"
#include "AutoRTFM.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PreprocessorHelpers.h"
#include "HAL/ThreadSafeCounter.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Build.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryLayout.h"
#include "Templates/Requires.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include <atomic>
#include <type_traits>

/**
 * Simple wrapper class which holds a refcount; emits a deprecation warning when accessed.
 * 
 * It is unsafe to rely on the value of a refcount for any logic, and a non-deprecated
 * getter function should never be added. In a multi-threaded context, the refcount could
 * change after inspection. In a transactional context, the refcount may be higher than
 * expected, as releases are deferred until completion of the transaction.
 */
struct FReturnedRefCountValue
{
	explicit FReturnedRefCountValue(uint32 InRefCount)
		: RefCount(InRefCount)
	{
	}
	FReturnedRefCountValue(const FReturnedRefCountValue& Other) = default;
	FReturnedRefCountValue(FReturnedRefCountValue&& Other) = default;
	FReturnedRefCountValue& operator=(const FReturnedRefCountValue& Other) = default;
	FReturnedRefCountValue& operator=(FReturnedRefCountValue&& Other) = default;

	UE_DEPRECATED(5.8, "Inspecting an object's refcount is deprecated.")
	operator uint32() const
	{
		return RefCount;
	}

	void CheckAtLeast(uint32 N) const
	{
		// It's harmless to check if your refcount is at least a certain amount. Be aware 
		// that inside an AutoRTFM transaction, Release() is deferred until commit, so an
		// object's refcount may be higher than you expected. In other words, when inside
		// of a transaction, this check may not trigger even when the number of active
		// reference holders is lower than the passed-in value.
		checkSlow(RefCount >= N);
	}

private:
	uint32 RefCount = 0;
};

namespace UE::Private
{

/**
 * Please do not use this, long term we want it to go away. Being able to query the reference count can
 * easily lead into really gnarly bugs and we generally don't want this sort of paradigm to be used more
 * widely. But there are a few places where it is currently being legitimately used so we'll provide a
 * queryable variant to let us deprecate the return of the reference count from `FRefCountedObject` asap.
 */
class FQueryableRefCountedObject : UE::Private::TAtomicRefCount<uint32, UE::Private::ERefCountIsQueryable::Yes>
{
public:
	FQueryableRefCountedObject() = default;
	virtual ~FQueryableRefCountedObject() = default;

	FQueryableRefCountedObject(const FQueryableRefCountedObject& Rhs) = delete;
	FQueryableRefCountedObject& operator=(const FQueryableRefCountedObject& Rhs) = delete;

	uint32 AddRef() const
	{
		return Super::AddRef<DeleteThis>();
	}

	uint32 Release() const
	{
		return Super::Release<DeleteThis>();
	}

	uint32 GetRefCount() const
	{
		return Super::GetRefCount();
	}

private:
	using Super = UE::Private::TAtomicRefCount<uint32, UE::Private::ERefCountIsQueryable::Yes>;

	static void DeleteThis(const Super* This)
	{
		delete static_cast<const FQueryableRefCountedObject*>(This);
	}
};
}

/** A virtual interface for ref counted objects to implement. */
class IRefCountedObject
{
public:
	virtual ~IRefCountedObject() = default;
	virtual void AddRef() const = 0;

	virtual FReturnedRefCountValue Release() const = 0;

	virtual FReturnedRefCountValue GetRefCount() const = 0;
};

/**
 * Base class implementing thread-safe reference counting.
 */
class FRefCountedObject : UE::Private::TAtomicRefCount<uint32>
{
public:
	FRefCountedObject() = default;
	virtual ~FRefCountedObject() = default;

	FRefCountedObject(const FRefCountedObject& Rhs) = delete;
	FRefCountedObject& operator=(const FRefCountedObject& Rhs) = delete;

	void AddRef() const
	{
		Super::AddRef<DeleteThis>();
	}

	FReturnedRefCountValue Release() const
	{
		return FReturnedRefCountValue{ Super::Release<DeleteThis>() };
	}

	FReturnedRefCountValue GetRefCount() const
	{
		return FReturnedRefCountValue{ Super::GetRefCount() };
	}

private:
	using Super = UE::Private::TAtomicRefCount<uint32>;

	static void DeleteThis(const Super* This)
	{
		delete static_cast<const FRefCountedObject*>(This);
	}
};

class UE_DEPRECATED(5.8, "`FRefCountedObject` is now thread-safe so use that instead.") FRefCountBase : public FRefCountedObject {};

class UE_DEPRECATED(5.8, "`FRefCountedObject` is now thread-safe so use that instead.") FThreadSafeRefCountedObject : public FRefCountedObject {};

/**
 * ERefCountingMode is used select between either 'fast' or 'thread safe' ref-counting types.
 * This is only used at compile time to select between template specializations.
 */
enum class ERefCountingMode : uint8
{
	/** Forced to be not thread-safe. */
	NotThreadSafe = 0,

	/** Thread-safe: never spin locks, but slower. */
	ThreadSafe = 1
};

/**
 * Ref-counting mixin, designed to add ref-counting to an object without requiring a virtual destructor.
 * Implements support for AutoRTFM, is thread-safe by default, and can support custom deleters via T::StaticDestroyObject.
 * 
 * @note AutoRTFM means that the return value of AddRef/Release can't be trusted (as the ref-count doesn't decrement until
 *       the transaction is committed), but this is fine for use with TRefCountPtr, as it doesn't use those return values.
 * 
 * Basic Example:
 *  struct FMyRefCountedObject : public TRefCountingMixin<FMyRefCountedObject>
 *  {
 *      // ...
 *  };
 * 
 * Deleter Example:
 *  struct FMyRefCountedPooledObject : public TRefCountingMixin<FMyRefCountedPooledObject>
 *  {
 *      static void StaticDestroyObject(const FMyRefCountedPooledObject* Obj)
 *      {
 *          GPool->ReturnToPool(Obj);
 *      }
 *  };
 */
template <typename T, ERefCountingMode Mode = ERefCountingMode::ThreadSafe>
class TRefCountingMixin;

/**
 * Thread-safe specialization
 */
template <typename T>
class TRefCountingMixin<T, ERefCountingMode::ThreadSafe> : UE::Private::TAtomicRefCount<uint32>
{
public:
	TRefCountingMixin() = default;

	TRefCountingMixin(const TRefCountingMixin&) = delete;
	TRefCountingMixin& operator=(const TRefCountingMixin&) = delete;

	void AddRef() const
	{
		Super::template AddRef<StaticDestroyMixin>();
	}

	FReturnedRefCountValue Release() const
	{
		return FReturnedRefCountValue{ Super::template Release<StaticDestroyMixin>() };
	}

	FReturnedRefCountValue GetRefCount() const
	{
		return FReturnedRefCountValue{ Super::GetRefCount() };
	}

	static void StaticDestroyObject(const T* Obj)
	{
		delete Obj;
	}

private:
	using Super = UE::Private::TAtomicRefCount<uint32>;

	static void StaticDestroyMixin(const Super* This)
	{
		// This static_cast is traversing two levels of the class hierarchy.
		// We are casting from our parent class (TAtomicRefCount*) to our subclass (T*).
		T::StaticDestroyObject(static_cast<const T*>(This));
	}
};

/**
 * Not-thread-safe specialization
 */
template <typename T>
class TRefCountingMixin<T, ERefCountingMode::NotThreadSafe> 
{
public:
	TRefCountingMixin() = default;

	TRefCountingMixin(const TRefCountingMixin&) = delete;
	TRefCountingMixin& operator=(const TRefCountingMixin&) = delete;

	void AddRef() const
	{
		++RefCount;
	}

	FReturnedRefCountValue Release() const
	{
		checkSlow(RefCount > 0);

		if (--RefCount == 0)
		{
			StaticDestroyMixin(this);
		}

		// Note: TRefCountPtr doesn't use the return value
		return FReturnedRefCountValue{ 0 };
	}

	FReturnedRefCountValue GetRefCount() const
	{
		return FReturnedRefCountValue{ RefCount };
	}

	static void StaticDestroyObject(const T* Obj)
	{
		delete Obj;
	}

private:
	static void StaticDestroyMixin(const TRefCountingMixin* This)
	{
		T::StaticDestroyObject(static_cast<const T*>(This));
	}

	mutable uint32 RefCount = 0;
};

/**
 * A smart pointer to an object which implements AddRef/Release.
 */
template<typename ReferencedType>
class TRefCountPtr
{
	using ReferenceType = ReferencedType*;

public:
	UE_FORCEINLINE_HINT TRefCountPtr() = default;

	TRefCountPtr(ReferencedType* InReference, bool bAddRef = true)
	{
		Reference = InReference;
		if (Reference && bAddRef)
		{
			Reference->AddRef();
		}
	}

	TRefCountPtr(const TRefCountPtr& Copy)
	{
		Reference = Copy.Reference;
		if (Reference)
		{
			Reference->AddRef();
		}
	}

	template<typename CopyReferencedType>
	explicit TRefCountPtr(const TRefCountPtr<CopyReferencedType>& Copy)
	{
		Reference = static_cast<ReferencedType*>(Copy.GetReference());
		if (Reference)
		{
			Reference->AddRef();
		}
	}

	inline TRefCountPtr(TRefCountPtr&& Move)
	{
		Reference = Move.Reference;
		Move.Reference = nullptr;
	}

	template<typename MoveReferencedType>
	explicit TRefCountPtr(TRefCountPtr<MoveReferencedType>&& Move)
	{
		Reference = static_cast<ReferencedType*>(Move.GetReference());
		Move.Reference = nullptr;
	}

	~TRefCountPtr()
	{
		if (Reference)
		{
			Reference->Release();
		}
	}

	TRefCountPtr& operator=(ReferencedType* InReference)
	{
		if (Reference != InReference)
		{
			// Call AddRef before Release, in case the new reference is the same as the old reference.
			ReferencedType* OldReference = Reference;
			Reference = InReference;
			if (Reference)
			{
				Reference->AddRef();
			}
			if (OldReference)
			{
				OldReference->Release();
			}
		}
		return *this;
	}

	UE_FORCEINLINE_HINT TRefCountPtr& operator=(const TRefCountPtr& InPtr)
	{
		return *this = InPtr.Reference;
	}

	template<typename CopyReferencedType>
	UE_FORCEINLINE_HINT TRefCountPtr& operator=(const TRefCountPtr<CopyReferencedType>& InPtr)
	{
		return *this = InPtr.GetReference();
	}

	TRefCountPtr& operator=(TRefCountPtr&& InPtr)
	{
		if (this != &InPtr)
		{
			ReferencedType* OldReference = Reference;
			Reference = InPtr.Reference;
			InPtr.Reference = nullptr;
			if (OldReference)
			{
				OldReference->Release();
			}
		}
		return *this;
	}

	template<typename MoveReferencedType>
	TRefCountPtr& operator=(TRefCountPtr<MoveReferencedType>&& InPtr)
	{
		// InPtr is a different type (or we would have called the other operator), so we need not test &InPtr != this
		ReferencedType* OldReference = Reference;
		Reference = InPtr.Reference;
		InPtr.Reference = nullptr;
		if (OldReference)
		{
			OldReference->Release();
		}
		return *this;
	}

	UE_FORCEINLINE_HINT ReferencedType* operator->() const
	{
		return Reference;
	}

	UE_FORCEINLINE_HINT operator ReferenceType() const
	{
		return Reference;
	}

	inline ReferencedType** GetInitReference()
	{
		*this = nullptr;
		return &Reference;
	}

	UE_FORCEINLINE_HINT ReferencedType* GetReference() const
	{
		return Reference;
	}

	UE_FORCEINLINE_HINT friend bool IsValidRef(const TRefCountPtr& InReference)
	{
		return InReference.Reference != nullptr;
	}

	UE_FORCEINLINE_HINT bool IsValid() const
	{
		return Reference != nullptr;
	}

	UE_FORCEINLINE_HINT void SafeRelease()
	{
		*this = nullptr;
	}

	uint32 GetRefCount()
	{
		uint32 Result = 0;
		if (Reference)
		{
			Result = Reference->GetRefCount();
			check(Result > 0); // you should never have a zero ref count if there is a live ref counted pointer (*this is live)
		}
		return Result;
	}

	inline void Swap(TRefCountPtr& InPtr) // this does not change the reference count, and so is faster
	{
		ReferencedType* OldReference = Reference;
		Reference = InPtr.Reference;
		InPtr.Reference = OldReference;
	}

	void Serialize(FArchive& Ar)
	{
		ReferenceType PtrReference = Reference;
		Ar << PtrReference;
		if(Ar.IsLoading())
		{
			*this = PtrReference;
		}
	}

private:
	ReferencedType* Reference = nullptr;

	template <typename OtherType>
	friend class TRefCountPtr;

public:
	UE_FORCEINLINE_HINT bool operator==(const TRefCountPtr& B) const
	{
		return GetReference() == B.GetReference();
	}

	UE_FORCEINLINE_HINT bool operator==(ReferencedType* B) const
	{
		return GetReference() == B;
	}
};

ALIAS_TEMPLATE_TYPE_LAYOUT(template<typename T>, TRefCountPtr<T>, void*);

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
template<typename ReferencedType>
UE_FORCEINLINE_HINT bool operator==(ReferencedType* A, const TRefCountPtr<ReferencedType>& B)
{
	return A == B.GetReference();
}
#endif

template<typename ReferencedType>
UE_FORCEINLINE_HINT uint32 GetTypeHash(const TRefCountPtr<ReferencedType>& InPtr)
{
	return GetTypeHash(InPtr.GetReference());
}


template<typename ReferencedType>
FArchive& operator<<(FArchive& Ar,TRefCountPtr<ReferencedType>& Ptr)
{
	Ptr.Serialize(Ar);
	return Ar;
}

template <
	typename T,
	typename... TArgs
	UE_REQUIRES(!std::is_array_v<T>)
>
[[nodiscard]] inline TRefCountPtr<T> MakeRefCount(TArgs&&... Args)
{
	T* NewUnwrappedObject = new T(Forward<TArgs>(Args)...);

	// Set up `NewObject` in the open to avoid unnecessary (but harmless) OnAbort tasks.
	TRefCountPtr<T> NewObject;
	UE_AUTORTFM_OPEN
	{
		NewObject = NewUnwrappedObject;
	};
	return NewObject;
}
