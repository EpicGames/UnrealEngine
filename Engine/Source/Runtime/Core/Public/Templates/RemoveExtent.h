// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Removes one dimension of extents from an array type.
 */
template <typename T>           struct TRemoveExtent       { typedef T Type; };
template <typename T>           struct TRemoveExtent<T[]>  { typedef T Type; };
template <typename T, uint32 N> struct TRemoveExtent<T[N]> { typedef T Type; };
