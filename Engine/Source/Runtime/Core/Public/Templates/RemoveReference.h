// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * TRemoveReference<type> will remove any references from a type.
 */
template <typename T> struct TRemoveReference      { typedef T Type; };
template <typename T> struct TRemoveReference<T& > { typedef T Type; };
template <typename T> struct TRemoveReference<T&&> { typedef T Type; };
