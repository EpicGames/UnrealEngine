// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/EnableIf.h"
#include <type_traits>

/*-----------------------------------------------------------------------------
	Readability macro for a constraint in template definitions, future-proofed
	for C++ 20 concepts. Usage:

	template <
		typename T,
		typename U  // note - no trailing comma before the constraint
		UE_REQUIRES(std::is_integral_v<T> && sizeof(U) <= 4)
	>
	void IntegralUpTo32Bit(T Lhs, U Rhs)
	{
	}

	When a template is both declared and defined at once, UE_REQUIRES must be used.

	If a forward declaration is needed, UE_REQUIRES must be used to declare the function
	outside the scope, then UE_REQUIRES_DEFINITION should be used for the constraint on the
	function declaration.

	If a friend declaration is needed, a forward declaration using UE_REQUIRES must first be
	specified at the required scope, then UE_REQUIRES_FRIEND should be used for the constraint
	in the friendship declaration.  Usage:

	template <
		typename T
		UE_REQUIRES(trait_v<T>)
	>
	void Function(T Val);

	class FThing
	{
	private:
		template <
			typename T
			UE_REQUIRES_FRIEND(trait_v<T>)
		>
		friend void FuncB(T);

		template <typename T>
		static void Use(T);
	};

	template <
		typename T
		UE_REQUIRES_DEFINITION(trait_v<T>)
	>
	void Function(T Val)
	{
		Use(Val);
	}

	UE_REQUIRES_EXPR() wraps the effects of a requires expression, used to test the
	compilability of an expression based on a deduced template parameter.  Usage:

	template <
		typename LhsType,
		typename RhsType
		UE_REQUIRES(
			std::is_base_of_v<UObject, LhsType> &&
			std::is_base_of_v<UObject, RhsType> &&
			UE_REQUIRES_EXPR(std::declval<const LhsType*>() == std::declval<const RhsType*>())
		)
	>
	bool operator==(const TSmartPtr<LhsType>& Lhs, const TSmartPtr<RhsType>& Rhs)
	{
		return Lhs.Get() == Rhs.Get();
	}

	Unlike a C++20 requires expression, UE_REQUIRES_EXPR() can only be used in
	the body of a UE_REQUIRES() macro - standalone concept checks must still be
	done via something like a TModels-type concept.
 -----------------------------------------------------------------------------*/
#if __cplusplus < 202000 || (defined(__clang__) && ((!PLATFORM_APPLE && !PLATFORM_ANDROID && __clang_major__ == 16) || (PLATFORM_APPLE && __clang_major__ == 15) || (PLATFORM_ANDROID && __clang_major__ == 17)))
	// Clang 16 (or Apple Clang 15, or Android Clang 17) treats a UE_REQUIRES_FRIEND declaration as a separate function, causing ambiguous overload resolution,
	// so fall back to the non-concept implementation.
	//
	// https://github.com/llvm/llvm-project/issues/60749

	#define UE_REQUIRES(...) , std::enable_if_t<(__VA_ARGS__), int> = 0
	#define UE_REQUIRES_FRIEND(...) , std::enable_if_t<(__VA_ARGS__), int>
	#define UE_REQUIRES_DEFINITION(...) , std::enable_if_t<(__VA_ARGS__), int>
	#define UE_REQUIRES_EXPR(...) (!std::is_same_v<decltype(__VA_ARGS__), long long************>) // This is highly unlikely to be the type of any expression
#else
	namespace UE::Core::Private
	{
		// Only needed for the UE_REQUIRES macro to work, to allow for a trailing > token after the macro
		template <bool B>
		concept BoolIdentityConcept = B;
	}

	#define UE_REQUIRES(...) > requires (!!(__VA_ARGS__)) && UE::Core::Private::BoolIdentityConcept<true
	#define UE_REQUIRES_FRIEND(...) UE_REQUIRES(__VA_ARGS__)
	#define UE_REQUIRES_DEFINITION(...) UE_REQUIRES(__VA_ARGS__)
	#define UE_REQUIRES_EXPR(...) requires { (__VA_ARGS__); }
#endif

#define TEMPLATE_REQUIRES(...) UE_DEPRECATED_MACRO(5.5, "TEMPLATE_REQUIRES has been deprecated - please use UE_REQUIRES instead.") typename TEnableIf<__VA_ARGS__, int>::type = 0
