// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/IsFloatingPoint.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/IsSigned.h"
#include "Templates/IsIntegral.h"
#include "Templates/Requires.h"

/**
 * This file defines a few macros which can be used to restrict (via deprecation) ambiguous calls to functions expecting float params/return type.
 * This also handles mixing of argument types where at least one argument is a float/double and the rest can be promoted to the highest-precision type to infer the expected
 * type of the return value, or where non-float arguments are mixed in a function where the result should be the result of mixed arithmetic between the types.
 * 
 * For example:
 *  
 *  template< class T >
 *	static T MyFunc(const T X, const T Y)
 *	{
 *		return X < Y ? X : Y;
 *	}
 *  // Allows mixing of types without causing type ambiguity
 *  MIX_TYPES_2_ARGS(MyFunc);
 * 
 * 	static float Sin( float Value ) { return sinf(Value); }
 *	static double Sin( double Value ) { return sin(Value); }
 *
 *	static float Fmod(float X, float Y);
 *	static double Fmod(double X, double Y);
 *  // Disallows Fmod(int32, int32) because one argument must be a floating point type.
 *  // Promotes calls such as "Fmod(float, double)" or "Fmod(double, int)" to "double Fmod(double, double)" because double has the highest precision of the argument types.
 *	RESOLVE_FLOAT_AMBIGUITY_2_ARGS(Fmod);
 */


// Mixing any signed integral types with any other signed integral type results in same type as "X * Y", which is promoted to the type of the result of mixed arithmetic between the types
#define MIX_SIGNED_INTS_2_ARGS_ACTUAL(Func, OptionalMarkup) \
	template < \
		typename Arg1, \
		typename Arg2 \
		UE_REQUIRES( \
			!std::is_same_v<Arg1, Arg2> && \
			std::is_signed_v<Arg1> && std::is_integral_v<Arg1> && \
			std::is_signed_v<Arg2> && std::is_integral_v<Arg2> \
		) \
	> \
	static OptionalMarkup inline auto Func(Arg1 X, Arg2 Y) -> decltype(X * Y) \
	{ \
		using ArgType = decltype(X * Y); \
		return Func((ArgType)X, (ArgType)Y); \
	}

#define MIX_SIGNED_INTS_2_ARGS(Func)				MIX_SIGNED_INTS_2_ARGS_ACTUAL(Func,)
#define MIX_SIGNED_INTS_2_ARGS_CONSTEXPR(Func)		MIX_SIGNED_INTS_2_ARGS_ACTUAL(Func, constexpr)

// Mixing any signed integral types with any other signed integral type results in same type as "X * Y", which is promoted to the type of the result of mixed arithmetic between the types
#define MIX_SIGNED_TYPES_3_ARGS_ACTUAL(Func, OptionalMarkup) \
	template < \
		typename Arg1, \
		typename Arg2, \
		typename Arg3 \
		UE_REQUIRES( \
			( \
				!std::is_same_v<Arg1, Arg2>> || \
				!std::is_same_v<Arg2, Arg3>> || \
				!std::is_same_v<Arg1, Arg3>> \
			) && \
			std::is_signed_v<Arg1> && std::is_integral_v<Arg1> && \
			std::is_signed_v<Arg2> && std::is_integral_v<Arg2> && \
			std::is_signed_v<Arg3> && std::is_integral_v<Arg3> \
		) \
	> \
	static OptionalMarkup inline auto Func(Arg1 X, Arg2 Y, Arg3 Z) -> decltype(X * Y * Z) \
	{ \
		using ArgType = decltype(X * Y * Z); \
		return Func((ArgType)X, (ArgType)Y, (ArgType)Z); \
	}

#define MIX_SIGNED_INTS_3_ARGS(Func)				MIX_SIGNED_INTS_3_ARGS_ACTUAL(Func,)
#define MIX_SIGNED_INTS_3_ARGS_CONSTEXPR(Func)		MIX_SIGNED_INTS_3_ARGS_ACTUAL(Func, constexpr)


// Mixing float and double types with any other type results in same type as "X * Y", which is promoted to the floating point type with the highest precision of the argument types.
#define MIX_FLOATS_2_ARGS(Func) \
	template < \
		typename Arg1, \
		typename Arg2 \
		UE_REQUIRES((std::is_floating_point_v<Arg1> || std::is_floating_point_v<Arg2>) && !std::is_same_v<Arg1, Arg2>) \
	> \
	static inline auto Func(Arg1 X, Arg2 Y) -> decltype(X * Y) \
	{ \
		using ArgType = decltype(X * Y); \
		return Func((ArgType)X, (ArgType)Y); \
	}

// Mixing float and double types with any other type results in same type as "X * Y", which is promoted to the floating point type with the highest precision of the argument types.
#define MIX_FLOATS_3_ARGS(Func) \
	template < \
		typename Arg1, \
		typename Arg2, \
		typename Arg3 \
		UE_REQUIRES( \
			(std::is_floating_point_v<Arg1> || std::is_floating_point_v<Arg2> || std::is_floating_point_v<Arg3>) && \
			(!std::is_same_v<Arg1, Arg2> || !std::is_same_v<Arg2, Arg3> || !std::is_same_v<Arg1, Arg3>) \
		) \
	> \
	static inline auto Func(Arg1 X, Arg2 Y, Arg3 Z) -> decltype(X * Y * Z) \
	{ \
		using ArgType = decltype(X * Y * Z); \
		return Func((ArgType)X, (ArgType)Y, (ArgType)Z); \
	}

// Resolve ambiguity between multiple arguments to functions returning a floating point type.
// If no arguments are floating point, emits a warning and resolves to float by default.
// Otherwise promotes to the highest precision floating point type of the arugments.

#define RESOLVE_FLOAT_AMBIGUITY_2_ARGS(Func) \
	MIX_FLOATS_2_ARGS(Func);

#define RESOLVE_FLOAT_AMBIGUITY_3_ARGS(Func) \
	MIX_FLOATS_3_ARGS(Func);


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

// Mixing float and double types with any other type results in same type as "X * Y", which is promoted to the floating point type with the highest precision of the argument types.
#define MIX_FLOATS_TO_TYPE_2_ARGS(Func, ReturnType) \
	template < \
		typename Arg1, \
		typename Arg2 \
		UE_REQUIRES( \
			(std::is_floating_point_v<Arg1> || std::is_floating_point_v<Arg2>) && \
			!std::is_same_v<Arg1, Arg2> \
		) \
	> \
	static inline ReturnType Func(Arg1 X, Arg2 Y) \
	{ \
		using ArgType = decltype(X * Y); \
		return Func((ArgType)X, (ArgType)Y); \
	}

// Mixing float and double types with any other type results in same type as "X * Y", which is promoted to the floating point type with the highest precision of the argument types.
#define MIX_FLOATS_TO_TYPE_3_ARGS(Func, ReturnType) \
	template < \
		typename Arg1, \
		typename Arg2, \
		typename Arg3 \
		UE_REQUIRES( \
			(std::is_floating_point_v<Arg1> || std::is_floating_point_v<Arg2> || std::is_floating_point_v<Arg3>) && \
			(!std::is_same_v<Arg1, Arg2> || !std::is_same_v<Arg1, Arg3> || !std::is_same_v<Arg2, Arg3>) \
		) \
	> \
	static inline ReturnType Func(Arg1 X, Arg2 Y, Arg3 Z) \
	{ \
		using ArgType = decltype(X * Y * Z); \
		return Func((ArgType)X, (ArgType)Y, (ArgType)Z); \
	}

// Resolve ambiguity between multiple arguments to functions returning a generic point type.
// If no arguments are floating point, emits a warning and resolves to float by default.
// Otherwise promotes to the highest precision floating point type of the arugments.

#define RESOLVE_FLOAT_TO_TYPE_AMBIGUITY_2_ARGS(Func, ReturnType) \
	MIX_FLOATS_TO_TYPE_2_ARGS(Func, ReturnType);

#define RESOLVE_FLOAT_TO_TYPE_AMBIGUITY_3_ARGS(Func, ReturnType) \
	MIX_FLOATS_TO_TYPE_3_ARGS(Func, ReturnType);

// nicer names for the bool version
#define RESOLVE_FLOAT_PREDICATE_AMBIGUITY_2_ARGS(Func) RESOLVE_FLOAT_TO_TYPE_AMBIGUITY_2_ARGS(Func, bool)
#define RESOLVE_FLOAT_PREDICATE_AMBIGUITY_3_ARGS(Func) RESOLVE_FLOAT_TO_TYPE_AMBIGUITY_3_ARGS(Func, bool)
