// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * TRetainedRef<T>
 *
 * A helper class which replaces T& as a function parameter when the reference is
 * intended to be retained by the function (e.g. as a class member).  The benefit
 * of this class is that it is a compile error to pass an rvalue which might otherwise
 * bind to a const reference, which is dangerous when the reference is retained.
 *
 * Example:
 *
 * struct FRaiiType
 * {
 *     explicit FRaiiType(const FThing& InThing)
 *         : Ref(InThing)
 *     {
 *     }
 *
 *     void DoSomething()
 *     {
 *         Ref.Something();
 *     }
 *
 *     FThing& Ref;
 * };
 *
 * void Test()
 * {
 *     FThing Thing(...);
 *     FRaiiType Raii1(Thing);       // Compiles
 *     Raii.DoSomething();           // Fine
 *
 *     FRaiiType Raii2(FThing(...)); // Compiles
 *     Raii.DoSomething();           // Illegal - reference has expired!
 * }
 *
 * But if the constructor is changed to use TRetainedRef then it fixes that problem:
 *
 * struct FRaiiType
 * {
 *     explicit FRaiiType(TRetainedRef<const FThing> InThing)
 *         : Ref(InThing)
 *     {
 *     }
 *
 *     ...
 * }
 *
 * FThing Thing(...);
 * FRaiiType Raii1(Thing);       // Compiles
 * Raii.DoSomething();           // Fine
 *
 * FRaiiType Raii2(FThing(...)); // Compile error!
 * Raii.DoSomething();
 */

template <typename T>
struct TRetainedRef
{
	TRetainedRef(T& InRef)
		: Ref(InRef)
	{
	}

	// Can't construct a non-const reference with a const reference
	// and can't retain a rvalue reference.
	TRetainedRef(const T&  InRef) = delete;
	TRetainedRef(      T&& InRef) = delete;
	TRetainedRef(const T&& InRef) = delete;

	operator T&() const
	{
		return Ref;
	}

	T& Get() const
	{
		return Ref;
	}

private:
	T& Ref;
};

template <typename T>
struct TRetainedRef<const T>
{
	TRetainedRef(T& InRef)
		: Ref(InRef)
	{
	}

	TRetainedRef(const T& InRef)
		: Ref(InRef)
	{
	}

	// Can't retain a rvalue reference.
	TRetainedRef(      T&& InRef) = delete;
	TRetainedRef(const T&& InRef) = delete;

	operator const T&() const
	{
		return Ref;
	}

	const T& Get() const
	{
		return Ref;
	}

private:
	const T& Ref;
};
