// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Invoke.h"
#include "Templates/UnrealTemplate.h"

/**
 * Helper class to reverse a predicate.
 * Performs Predicate(B, A)
 */
template <typename PredicateType>
class TReversePredicate
{
	const PredicateType& Predicate;

public:
	TReversePredicate( const PredicateType& InPredicate )
		: Predicate( InPredicate )
	{
	}

	template <typename T>
	UE_FORCEINLINE_HINT bool operator()( T&& A, T&& B ) const
	{
		return ::Invoke( Predicate, Forward<T>(B), Forward<T>(A) );
	}
};
