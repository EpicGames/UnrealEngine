// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"


/**
 * Helper object for batching callback requests and firing on destruction of the FScopedCallback object.
 * CallbackType is a class implementing a static method called FireCallback, which does the work.
 */
template< class CallbackType >
class TScopedCallback
{
public:

	TScopedCallback()
		:	Counter(0)
	{ }

	/** Fires a callback if outstanding requests exist. */
	~TScopedCallback()
	{
		if (HasRequests())
		{
			CallbackType::FireCallback();
		}
	}

	/** Request a callback. */
	void Request()
	{
		++Counter;
	}

	/** Unrequest a callback. */
	void Unrequest()
	{
		--Counter;
	}

	/**
	 * Checks whether this callback has outstanding requests.
	 *
	 * @return true if there are outstanding requests, false otherwise.
	 */
	bool HasRequests() const
	{
		return Counter > 0;
	}

private:

	/** Counts callback requests. */
	int32	Counter;
};
