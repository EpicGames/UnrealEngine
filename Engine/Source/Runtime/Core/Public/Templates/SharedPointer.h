// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "AutoRTFM/ReturnFromOpenMode.h"
#include "Misc/IntrusiveUnsetOptionalState.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreGlobals.h"
#include "Misc/UEOps.h"


/**
 *	SharedPointer - Unreal smart pointer library
 *
 *	This is a smart pointer library consisting of shared references (TSharedRef), shared pointers (TSharedPtr),
 *	weak pointers (TWeakPtr) as well as related helper functions and classes.  This implementation is modeled
 *	after the C++0x standard library's shared_ptr as well as Boost smart pointers.
 *
 *	Benefits of using shared references and pointers:
 *
 *		Clean syntax.  You can copy, dereference and compare shared pointers just like regular C++ pointers.
 *		Prevents memory leaks.  Resources are destroyed automatically when there are no more shared references.
 *		Weak referencing.  Weak pointers allow you to safely check when an object has been destroyed.
 *		Thread safety.  Includes "thread safe" version that can be safely accessed from multiple threads.
 *		Ubiquitous.  You can create shared pointers to virtually *any* type of object.
 *		Runtime safety.  Shared references are never null and can always be dereferenced.
 *		No reference cycles.  Use weak pointers to break reference cycles.
 *		Confers intent.  You can easily tell an object *owner* from an *observer*.
 *		Performance.  Shared pointers have minimal overhead.  All operations are constant-time.
 *		Robust features.  Supports 'const', forward declarations to incomplete types, type-casting, etc.
 *		Memory.  Only twice the size of a C++ pointer in 64-bit (plus a shared 16-byte reference controller.)
 *
 *
 *	This library contains the following smart pointers:
 *
 *		TSharedRef - Non-nullable, reference counted non-intrusive authoritative smart pointer
 *		TSharedPtr - Reference counted non-intrusive authoritative smart pointer
 *		TWeakPtr - Reference counted non-intrusive weak pointer reference
 *
 *
 *	Additionally, the following helper classes and functions are defined:
 *
 *		MakeShareable() - Used to initialize shared pointers from C++ pointers (enables implicit conversion)
 *		MakeShared<T>(...) - Used to construct a T alongside its controller, saving an allocation.
 *		TSharedFromThis - You can derive your own class from this to acquire a TSharedRef from "this"
 *		StaticCastSharedRef() - Static cast utility function, typically used to downcast to a derived type. 
 *		ConstCastSharedRef() - Converts a 'const' reference to 'mutable' smart reference
 *		StaticCastSharedPtr() - Dynamic cast utility function, typically used to downcast to a derived type. 
 *		ConstCastSharedPtr() - Converts a 'const' smart pointer to 'mutable' smart pointer
 *		StaticCastWeakPtr() - Dynamic cast utility function, typically used to downcast to a derived type.
 *		ConstCastWeakPtr() - Converts a 'const' smart pointer to 'mutable' smart pointer
 *
 *
 *	Examples:
 *		- Please see 'SharedPointerTesting.inl' for various examples of shared pointers and references!
 *
 *
 *	Tips:
 *		- Use TSharedRef instead of TSharedPtr whenever possible -- it can never be nullptr!
 *		- You can call TSharedPtr::Reset() to release a reference to your object (and potentially deallocate) 
 *		- Use the MakeShareable() helper function to implicitly convert to TSharedRefs or TSharedPtrs
 *		- Prefer MakeShared<T>(...) to MakeShareable(new T(...))
 *		- You can never reset a TSharedRef or assign it to nullptr, but you can assign it a new object
 *		- Shared pointers assume ownership of objects -- no need to call delete yourself!
 *		- Usually you should "operator new" when passing a C++ pointer to a new shared pointer
 *		- Use TSharedRef or TSharedPtr when passing smart pointers as function parameters, not TWeakPtr
 *		- The "thread-safe" versions of smart pointers are a bit slower -- only use them when needed
 *		- You can forward declare shared pointers to incomplete types, just how you'd expect to!
 *		- Shared pointers of compatible types will be converted implicitly (e.g. upcasting)
 *		- You can create a typedef to TSharedRef< MyClass > to make it easier to type
 *		- For best performance, minimize calls to TWeakPtr::Pin (or conversions to TSharedRef/TSharedPtr)
 *		- Your class can return itself as a shared reference if you derive from TSharedFromThis
 *		- To downcast a pointer to a derived object class, use the StaticCastSharedRef, StaticCastSharedRef or StaticCastWeakPtr functions
 *		- 'const' objects are fully supported with shared pointers!
 *		- You can make a 'const' pointer mutable using the ConstCastSharedRef, ConstCastSharedPtr or ConstCastWeakPtr functions
 *		
 *
 *	Limitations:
 *
 *		- Shared pointers are not compatible with Unreal objects (UObject classes)!
 *		- Currently only types with that have regular destructors (no custom deleters)
 *		- Dynamically-allocated arrays are not supported yet (e.g. MakeShareable( new int32[20] ))
 *
 *
 *	Differences from other implementations (e.g. boost:shared_ptr, std::shared_ptr):
 *
 *		- Type names and method names are more consistent with Unreal's codebase
 *		- You must use Pin() to convert weak pointers to shared pointers (no explicit constructor)
 *		- Thread-safety features are optional instead of forced
 *		- TSharedFromThis returns a shared *reference*, not a shared *pointer*
 *		- Some features were omitted (e.g. use_count(), unique(), etc.)
 *		- No exceptions are allowed (all related features have been omitted)
 *		- Custom allocators and custom delete functions are not supported yet
 *		- Our implementation supports non-nullable smart pointers (TSharedRef)
 *		- Several other new features added, such as MakeShareable and nullptr assignment
 *
 *
 *	Why did we write our own Unreal shared pointer instead of using available alternatives?
 *
 *		- std::shared_ptr (and even tr1::shared_ptr) is not yet available on all platforms
 *		- Allows for a more consistent implementation on all compilers and platforms
 *		- Can work seamlessly with other Unreal containers and types
 *		- Better control over platform specifics, including threading and optimizations
 *		- We want thread-safety features to be optional (for performance)
 *		- We've added our own improvements (MakeShareable, assign to nullptr, etc.)
 *		- Exceptions were not needed nor desired in our implementation
 *		- We wanted more control over performance (inlining, memory, use of virtuals, etc.)
 *		- Potentially easier to debug (liberal code comments, etc.)
 *		- Prefer not to introduce new third party dependencies when not needed
 *
 */

// SharedPointerInternals.h contains the implementation of reference counting structures we need
#include "Templates/SharedPointerInternals.h" // IWYU pragma: export


/**
 * Casts a shared reference of one type to another type. (static_cast)  Useful for down-casting.
 *
 * @param  InSharedRef  The shared reference to cast
 */
template< class CastToType, class CastFromType, ESPMode Mode >
[[nodiscard]] UE_FORCEINLINE_HINT TSharedRef< CastToType, Mode > StaticCastSharedRef( TSharedRef< CastFromType, Mode > const& InSharedRef )
{
	return TSharedRef< CastToType, Mode >( InSharedRef, SharedPointerInternals::FStaticCastTag() );
}


namespace UE::Core::Private
{
	// Needed to work around an Android compiler bug - we need to construct a TSharedRef
	// from MakeShared without making MakeShared a friend in order to access the private constructor.
	template <typename ObjectType, ESPMode Mode>
	UE_FORCEINLINE_HINT TSharedRef<ObjectType, Mode> MakeSharedRef(ObjectType* InObject, SharedPointerInternals::TReferenceControllerBase<Mode>* InSharedReferenceCount)
	{
		return TSharedRef<ObjectType, Mode>(InObject, InSharedReferenceCount);
	}
}


/**
 * TSharedRef is a non-nullable, non-intrusive reference-counted authoritative object reference.
 *
 * This shared reference will be conditionally thread-safe when the optional Mode template argument is set to ThreadSafe.
 */
// NOTE: TSharedRef is an Unreal extension to standard smart pointer feature set
template< class ObjectType, ESPMode InMode >
class TSharedRef
{
public:
	static_assert(!std::is_void_v<ObjectType>, "TSharedRef<void> is not supported - use TSharedPtr instead");

	using ElementType = ObjectType;
	static constexpr ESPMode Mode = InMode;

	// NOTE: TSharedRef has no default constructor as it does not support empty references.  You must
	//		 initialize your TSharedRef to a valid object at construction time.

	/**
	 * Constructs a shared reference that owns the specified object.  Must not be nullptr.
	 *
	 * @param  InObject  Object this shared reference to retain a reference to
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG explicit TSharedRef( OtherType* InObject )
		: Object( InObject )
		, SharedReferenceCount( SharedPointerInternals::NewDefaultReferenceController< Mode >( InObject ) )
	{
		Init(InObject);
	}

	/**
	 * Constructs a shared reference that owns the specified object.  Must not be nullptr.
	 *
	 * @param  InObject   Object this shared pointer to retain a reference to
	 * @param  InDeleter  Deleter object used to destroy the object when it is no longer referenced.
	 */
	template <
		typename OtherType,
		typename DeleterType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TSharedRef( OtherType* InObject, DeleterType&& InDeleter )
		: Object( InObject )
		, SharedReferenceCount( SharedPointerInternals::NewCustomReferenceController< Mode >( InObject, Forward< DeleterType >( InDeleter ) ) )
	{
		Init(InObject);
	}

	/**
	 * Constructs default shared reference that owns the default object for specified type.
	 *
	 * Used internally only. Please do not use!
	 */
	TSharedRef()
		: Object(new ObjectType())
		, SharedReferenceCount(SharedPointerInternals::NewDefaultReferenceController<Mode>(Object))
	{
		EnsureRetrievingVTablePtrDuringCtor(TEXT("TSharedRef()"));
		Init(Object);
	}

	/**
	 * Constructs a shared reference using a proxy reference to a raw pointer. (See MakeShareable())
	 * Must not be nullptr.
	 *
	 * @param  InRawPtrProxy  Proxy raw pointer that contains the object that the new shared reference will reference
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TSharedRef( SharedPointerInternals::TRawPtrProxy< OtherType > const& InRawPtrProxy )
		: Object( InRawPtrProxy.Object )
		, SharedReferenceCount( SharedPointerInternals::NewDefaultReferenceController< Mode >( InRawPtrProxy.Object ) )
	{
		// If the following assert goes off, it means a TSharedRef was initialized from a nullptr object pointer.
		// Shared references must never be nullptr, so either pass a valid object or consider using TSharedPtr instead.
		check( InRawPtrProxy.Object != nullptr );

		// If the object happens to be derived from TSharedFromThis, the following method
		// will prime the object with a weak pointer to itself.
		SharedPointerInternals::EnableSharedFromThis(SharedReferenceCount, InRawPtrProxy.Object);
	}

	/**
	 * Constructs a shared reference using a proxy reference to a raw pointer. (See MakeShareable())
	 * Must not be nullptr.
	 *
	 * @param  InRawPtrProxy  Proxy raw pointer that contains the object that the new shared reference will reference
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <
		typename OtherType,
		typename DeleterType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TSharedRef( SharedPointerInternals::TRawPtrProxyWithDeleter< OtherType, DeleterType > const& InRawPtrProxy )
		: Object( InRawPtrProxy.Object )
		, SharedReferenceCount( SharedPointerInternals::NewCustomReferenceController< Mode >( InRawPtrProxy.Object, InRawPtrProxy.Deleter ) )
	{
		// If the following assert goes off, it means a TSharedRef was initialized from a nullptr object pointer.
		// Shared references must never be nullptr, so either pass a valid object or consider using TSharedPtr instead.
		check( InRawPtrProxy.Object != nullptr );

		// If the object happens to be derived from TSharedFromThis, the following method
		// will prime the object with a weak pointer to itself.
		SharedPointerInternals::EnableSharedFromThis( this, InRawPtrProxy.Object );
	}

	/**
	 * Constructs a shared reference using a proxy reference to a raw pointer. (See MakeShareable())
	 * Must not be nullptr.
	 *
	 * @param  InRawPtrProxy  Proxy raw pointer that contains the object that the new shared reference will reference
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <
		typename OtherType,
		typename DeleterType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TSharedRef( SharedPointerInternals::TRawPtrProxyWithDeleter< OtherType, DeleterType >&& InRawPtrProxy )
		: Object( InRawPtrProxy.Object )
		, SharedReferenceCount( SharedPointerInternals::NewCustomReferenceController< Mode >( InRawPtrProxy.Object, MoveTemp( InRawPtrProxy.Deleter ) ) )
	{
		// If the following assert goes off, it means a TSharedRef was initialized from a nullptr object pointer.
		// Shared references must never be nullptr, so either pass a valid object or consider using TSharedPtr instead.
		check( InRawPtrProxy.Object != nullptr );

		// If the object happens to be derived from TSharedFromThis, the following method
		// will prime the object with a weak pointer to itself.
		SharedPointerInternals::EnableSharedFromThis(SharedReferenceCount, InRawPtrProxy.Object);
	}

	/**
	 * Constructs a shared reference as a reference to an existing shared reference's object.
	 * This constructor is needed so that we can implicitly upcast to base classes.
	 *
	 * @param  InSharedRef  The shared reference whose object we should create an additional reference to
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TSharedRef( TSharedRef< OtherType, Mode > const& InSharedRef )
		: Object( InSharedRef.Object )
		, SharedReferenceCount( InSharedRef.SharedReferenceCount )
	{
	}

	/**
	 * Special constructor used internally to statically cast one shared reference type to another.  You
	 * should never call this constructor directly.  Instead, use the StaticCastSharedRef() function.
	 * This constructor creates a shared reference as a shared reference to an existing shared reference after
	 * statically casting that reference's object.  This constructor is needed for static casts.
	 *
	 * @param  InSharedRef  The shared reference whose object we should create an additional reference to
	 */
	template <typename OtherType>
	UE_NODEBUG inline TSharedRef( TSharedRef< OtherType, Mode > const& InSharedRef, SharedPointerInternals::FStaticCastTag )
		: Object( static_cast< ObjectType* >( InSharedRef.Object ) )
		, SharedReferenceCount( InSharedRef.SharedReferenceCount )
	{
	}

	/**
	 * Special constructor used internally to cast a 'const' shared reference a 'mutable' reference.  You
	 * should never call this constructor directly.  Instead, use the ConstCastSharedRef() function.
	 * This constructor creates a shared reference as a shared reference to an existing shared reference after
	 * const casting that reference's object.  This constructor is needed for const casts.
	 *
	 * @param  InSharedRef  The shared reference whose object we should create an additional reference to
	 */
	template <typename OtherType>
	UE_NODEBUG inline TSharedRef( TSharedRef< OtherType, Mode > const& InSharedRef, SharedPointerInternals::FConstCastTag )
		: Object( const_cast< ObjectType* >( InSharedRef.Object ) )
		, SharedReferenceCount( InSharedRef.SharedReferenceCount )
	{
	}

	/**
	 * Aliasing constructor used to create a shared reference which shares its reference count with
	 * another shared object, but pointing to a different object, typically a subobject.
	 *
	 * @param  OtherSharedRef  The shared reference whose reference count should be shared.
	 * @param  InObject  The object pointer to use (instead of the incoming shared pointer's object)
	 */
	template <typename OtherType>
	UE_NODEBUG inline TSharedRef( TSharedRef< OtherType, Mode > const& OtherSharedRef, ObjectType* InObject )
		: Object( InObject )
		, SharedReferenceCount( OtherSharedRef.SharedReferenceCount )
	{
		// If the following assert goes off, it means a TSharedRef was initialized from a nullptr object pointer.
		// Shared references must never be nullptr, so either pass a valid object or consider using TSharedPtr instead.
		check( InObject != nullptr );
	}

	UE_NODEBUG inline TSharedRef( TSharedRef const& InSharedRef )
		: Object( InSharedRef.Object )
		, SharedReferenceCount( InSharedRef.SharedReferenceCount )
	{
	}

	UE_NODEBUG inline TSharedRef( TSharedRef&& InSharedRef )
		: Object( InSharedRef.Object )
		, SharedReferenceCount( InSharedRef.SharedReferenceCount )
	{
		// We're intentionally not moving here, because we don't want to leave InSharedRef in a
		// null state, because that breaks the class invariant.  But we provide a move constructor
		// anyway in case the compiler complains that we have a move assign but no move construct.
	}

	/**
	 * Assignment operator replaces this shared reference with the specified shared reference.  The object
	 * currently referenced by this shared reference will no longer be referenced and will be deleted if
	 * there are no other referencers.
	 *
	 * @param  InSharedRef  Shared reference to replace with
	 */
	UE_NODEBUG inline TSharedRef& operator=( TSharedRef const& InSharedRef )
	{
		TSharedRef Temp = InSharedRef;
		::Swap(Temp, *this);
		return *this;
	}

	UE_REWRITE TSharedRef& operator=( TSharedRef&& InSharedRef )
	{
		::Swap(*this, InSharedRef);
		return *this;
	}

	/**
	 * Assignment operator replaces this shared reference with the specified shared reference.  The object
	 * currently referenced by this shared reference will no longer be referenced and will be deleted if
	 * there are no other referencers.  Must not be nullptr.
	 *
	 * @param  InRawPtrProxy  Proxy object used to assign the object (see MakeShareable helper function)
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TSharedRef& operator=( SharedPointerInternals::TRawPtrProxy< OtherType > const& InRawPtrProxy )
	{
		// If the following assert goes off, it means a TSharedRef was initialized from a nullptr object pointer.
		// Shared references must never be nullptr, so either pass a valid object or consider using TSharedPtr instead.
		check( InRawPtrProxy.Object != nullptr );

		*this = TSharedRef< ObjectType, Mode >( InRawPtrProxy );
		return *this;
	}

	/**
	 * Assignment operator replaces this shared reference with the specified shared reference.  The object
	 * currently referenced by this shared reference will no longer be referenced and will be deleted if
	 * there are no other referencers.  Must not be nullptr.
	 *
	 * @param  InRawPtrProxy  Proxy object used to assign the object (see MakeShareable helper function)
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <
		typename OtherType,
		typename DeleterType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TSharedRef& operator=( SharedPointerInternals::TRawPtrProxyWithDeleter< OtherType, DeleterType > const& InRawPtrProxy )
	{
		// If the following assert goes off, it means a TSharedRef was initialized from a nullptr object pointer.
		// Shared references must never be nullptr, so either pass a valid object or consider using TSharedPtr instead.
		check( InRawPtrProxy.Object != nullptr );

		*this = TSharedRef< ObjectType, Mode >( InRawPtrProxy );
		return *this;
	}

	/**
	 * Assignment operator replaces this shared reference with the specified shared reference.  The object
	 * currently referenced by this shared reference will no longer be referenced and will be deleted if
	 * there are no other referencers.  Must not be nullptr.
	 *
	 * @param  InRawPtrProxy  Proxy object used to assign the object (see MakeShareable helper function)
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <
		typename OtherType,
		typename DeleterType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TSharedRef& operator=( SharedPointerInternals::TRawPtrProxyWithDeleter< OtherType, DeleterType >&& InRawPtrProxy )
	{
		// If the following assert goes off, it means a TSharedRef was initialized from a nullptr object pointer.
		// Shared references must never be nullptr, so either pass a valid object or consider using TSharedPtr instead.
		check( InRawPtrProxy.Object != nullptr );

		*this = TSharedRef< ObjectType, Mode >( MoveTemp( InRawPtrProxy ) );
		return *this;
	}

	/** TSharedRef can be returned from AutoRTFM::Open() via a copy-construction in the closed. */
	static constexpr AutoRTFM::EReturnFromOpenMode AutoRTFMReturnFromOpenMode = AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed;

	/**
	 * Converts a shared reference to a shared pointer.
	 *
	 * @return  Shared pointer to the object
	 */
	[[nodiscard]] UE_REWRITE TSharedPtr<ObjectType, Mode> ToSharedPtr() const
	{
		return TSharedPtr<ObjectType, Mode>(*this);
	}

	/**
	 * Converts a shared reference to a weak ptr.
	 *
	 * @return  Weak pointer to the object
	 */
	[[nodiscard]] UE_REWRITE TWeakPtr<ObjectType, Mode> ToWeakPtr() const
	{
		return TWeakPtr<ObjectType, Mode>(*this);
	}

	/**
	 * Returns a C++ reference to the object this shared reference is referencing
	 *
	 * @return  The object owned by this shared reference
	 */
	[[nodiscard]] UE_REWRITE ObjectType& Get() const
	{
		// Should never be nullptr as TSharedRef is never nullable
		checkSlow( IsValid() );
		return *Object;
	}

	/**
	 * Dereference operator returns a reference to the object this shared pointer points to
	 *
	 * @return  Reference to the object
	 */
	[[nodiscard]] UE_REWRITE ObjectType& operator*() const
	{
		// Should never be nullptr as TSharedRef is never nullable
		checkSlow( IsValid() );
		return *Object;
	}

	/**
	 * Arrow operator returns a pointer to this shared reference's object
	 *
	 * @return  Returns a pointer to the object referenced by this shared reference
	 */
	[[nodiscard]] UE_REWRITE ObjectType* operator->() const
	{
		// Should never be nullptr as TSharedRef is never nullable
		checkSlow( IsValid() );
		return Object;
	}

	/**
	 * Returns the number of shared references to this object (including this reference.)
	 * IMPORTANT: Not necessarily fast!  Should only be used for debugging purposes!
	 *
	 * @return  Number of shared references to the object (including this reference.)
	 */
	[[nodiscard]] UE_REWRITE int32 GetSharedReferenceCount() const
	{
		return SharedReferenceCount.GetSharedReferenceCount();
	}

	/**
	 * Returns true if this is the only shared reference to this object.  Note that there may be
	 * outstanding weak references left.
	 *
	 * IMPORTANT: This has different behavior to GetSharedReferenceCount() == 1 in a multithreaded
	 *            context.  The expectation is that this will be used when a user wants exclusive
	 *            write-access to an otherwise-immutable object.  Care still needs to be taken when
	 *            pinning TWeakPtrs to make new shared references.
	 *
	 * @return  True if there is only one shared reference to the object, and this is it!
	 */
	[[nodiscard]] UE_REWRITE bool IsUnique() const
	{
		return SharedReferenceCount.IsUnique();
	}

	/////////////////////////////////////////////////////
	// Start - intrusive TOptional<TSharedRef> state //
	/////////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TSharedRef;

	UE_REWRITE explicit TSharedRef(FIntrusiveUnsetOptionalState)
		: Object(nullptr)
	{
	}
	UE_REWRITE bool UEOpEquals(FIntrusiveUnsetOptionalState) const
	{
		return !IsValid();
	}
	/////////////////////////////////////////////////
	// End - intrusive TOptional<TSharedRef> state //
	/////////////////////////////////////////////////

	template <typename OtherType>
	[[nodiscard]] UE_NODEBUG bool UEOpEquals(const TSharedRef<OtherType, Mode>& Rhs) const
	{
		return this->Object == &Rhs.Get();
	}

	template <typename OtherType>
	[[nodiscard]] UE_NODEBUG bool UEOpEquals(const TSharedPtr<OtherType, Mode>& Rhs) const
	{
		// This comparison against null is maintained as existing behavior, but isn't consistent with TWeakPtr comparison.
		OtherType* RhsPtr = Rhs.Get();
		return RhsPtr && RhsPtr == this->Object;
	}

	template <typename OtherType>
	[[nodiscard]] UE_NODEBUG bool UEOpEquals(const TWeakPtr<OtherType, Mode>& Rhs) const
	{
		return this->Object == Rhs.Pin().Get();
	}

	template <typename OtherType>
	[[nodiscard]] UE_NODEBUG bool UEOpLessThan(const TSharedRef<OtherType, Mode>& Rhs) const
	{
		return this->Object < &Rhs.Get();
	}

private:
	template<class OtherType>
	void Init(OtherType* InObject)
	{
		// If the following assert goes off, it means a TSharedRef was initialized from a nullptr object pointer.
		// Shared references must never be nullptr, so either pass a valid object or consider using TSharedPtr instead.
		check(InObject != nullptr);

		// If the object happens to be derived from TSharedFromThis, the following method
		// will prime the object with a weak pointer to itself.
		SharedPointerInternals::EnableSharedFromThis(SharedReferenceCount, InObject);
	}

	/**
	 * Converts a shared pointer to a shared reference.  The pointer *must* be valid or an assertion will trigger.
	 * NOTE: This explicit conversion constructor is intentionally private.  Use 'ToSharedRef()' instead.
	 *
	 * @return  Reference to the object
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline explicit TSharedRef( TSharedPtr< OtherType, Mode > const& InSharedPtr )
		: Object( InSharedPtr.Object )
		, SharedReferenceCount( InSharedPtr.SharedReferenceCount )
	{
		// If this assert goes off, it means a shared reference was created from a shared pointer that was nullptr.
		// Shared references are never allowed to be null.  Consider using TSharedPtr instead.
		check( IsValid() );
	}

	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline explicit TSharedRef( TSharedPtr< OtherType, Mode >&& InSharedPtr )
		: Object( InSharedPtr.Object )
		, SharedReferenceCount( MoveTemp(InSharedPtr.SharedReferenceCount) )
	{
		InSharedPtr.Object = nullptr;

		// If this assert goes off, it means a shared reference was created from a shared pointer that was nullptr.
		// Shared references are never allowed to be null.  Consider using TSharedPtr instead.
		check( IsValid() );
	}

	/**
	 * Checks to see if this shared reference is actually pointing to an object. 
	 * NOTE: This validity test is intentionally private because shared references must always be valid.
	 *
	 * @return  True if the shared reference is valid and can be dereferenced
	 */
	[[nodiscard]] UE_REWRITE bool IsValid() const
	{
		return Object != nullptr;
	}

	// We declare ourselves as a friend (templated using OtherType) so we can access members as needed
    template< class OtherType, ESPMode OtherMode > friend class TSharedRef;

	// Declare other smart pointer types as friends as needed
    template< class OtherType, ESPMode OtherMode > friend class TSharedPtr;
    template< class OtherType, ESPMode OtherMode > friend class TWeakPtr;

private:

	/** The object we're holding a reference to.  Can be nullptr. */
	ObjectType* Object;

	/** Interface to the reference counter for this object.  Note that the actual reference
		controller object is shared by all shared and weak pointers that refer to the object */
	SharedPointerInternals::FSharedReferencer< Mode > SharedReferenceCount;

	// VC emits an erroneous warning here - there is no inline specifier!
	#ifdef _MSC_VER
		#pragma warning(push)
		#pragma warning(disable : 4396) // warning: the inline specifier cannot be used when a friend declaration refers to a specialization of a function template
	#endif

	friend TSharedRef UE::Core::Private::MakeSharedRef<ObjectType, Mode>(ObjectType* InObject, SharedPointerInternals::TReferenceControllerBase<Mode>* InSharedReferenceCount);

	#ifdef _MSC_VER
		#pragma warning(pop)
	#endif

	UE_NODEBUG inline explicit TSharedRef(ObjectType* InObject, SharedPointerInternals::TReferenceControllerBase<Mode>* InSharedReferenceCount)
		: Object(InObject)
		, SharedReferenceCount(InSharedReferenceCount)
	{
		Init(InObject);
	}
};

/**
 * Trait which determines whether or not a type is a TSharedRef.
 */
template <typename T>                       constexpr bool TIsTSharedRef_V                                                = false;
template <class ObjectType, ESPMode InMode> constexpr bool TIsTSharedRef_V<               TSharedRef<ObjectType, InMode>> = true;
template <class ObjectType, ESPMode InMode> constexpr bool TIsTSharedRef_V<const          TSharedRef<ObjectType, InMode>> = true;
template <class ObjectType, ESPMode InMode> constexpr bool TIsTSharedRef_V<      volatile TSharedRef<ObjectType, InMode>> = true;
template <class ObjectType, ESPMode InMode> constexpr bool TIsTSharedRef_V<const volatile TSharedRef<ObjectType, InMode>> = true;

template <class ObjectType, ESPMode Mode>
struct TCallTraits<TSharedRef<ObjectType, Mode>> : public TCallTraitsBase<TSharedRef<ObjectType, Mode>>
{
	using ConstPointerType = TSharedRef<const ObjectType, Mode>;
};


/**
 * TSharedPtr is a non-intrusive reference-counted authoritative object pointer.  This shared pointer
 * will be conditionally thread-safe when the optional Mode template argument is set to ThreadSafe.
 */
template< class ObjectType, ESPMode InMode >
class TSharedPtr
{
public:
	using ElementType = ObjectType;
	static constexpr ESPMode Mode = InMode;

	/**
	 * Constructs an empty shared pointer
	 */
	// NOTE: FNullTag parameter is an Unreal extension to standard shared_ptr behavior
	UE_REWRITE TSharedPtr( SharedPointerInternals::FNullTag* = nullptr )
		: Object( nullptr )
		, SharedReferenceCount()
	{
	}

	/**
	 * Constructs a shared pointer that owns the specified object.  Note that passing nullptr here will
	 * still create a tracked reference to a nullptr pointer. (Consistent with std::shared_ptr)
	 *
	 * @param  InObject  Object this shared pointer to retain a reference to
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline explicit TSharedPtr( OtherType* InObject )
		: Object( InObject )
		, SharedReferenceCount( SharedPointerInternals::NewDefaultReferenceController< Mode >( InObject ) )
	{
		// If the object happens to be derived from TSharedFromThis, the following method
		// will prime the object with a weak pointer to itself.
		SharedPointerInternals::EnableSharedFromThis(SharedReferenceCount, InObject);
	}

	/**
	 * Constructs a shared pointer that owns the specified object.  Note that passing nullptr here will
	 * still create a tracked reference to a nullptr pointer. (Consistent with std::shared_ptr)
	 *
	 * @param  InObject   Object this shared pointer to retain a reference to
	 * @param  InDeleter  Deleter object used to destroy the object when it is no longer referenced.
	 */
	template <
		typename OtherType,
		typename DeleterType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TSharedPtr( OtherType* InObject, DeleterType&& InDeleter )
		: Object( InObject )
		, SharedReferenceCount( SharedPointerInternals::NewCustomReferenceController< Mode >( InObject, Forward< DeleterType >( InDeleter ) ) )
	{
		// If the object happens to be derived from TSharedFromThis, the following method
		// will prime the object with a weak pointer to itself.
		SharedPointerInternals::EnableSharedFromThis(SharedReferenceCount, InObject);
	}

	/**
	 * Constructs a shared pointer using a proxy reference to a raw pointer. (See MakeShareable())
	 *
	 * @param  InRawPtrProxy  Proxy raw pointer that contains the object that the new shared pointer will reference
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TSharedPtr( SharedPointerInternals::TRawPtrProxy< OtherType > const& InRawPtrProxy )
		: Object( InRawPtrProxy.Object )
		, SharedReferenceCount( SharedPointerInternals::NewDefaultReferenceController< Mode >( InRawPtrProxy.Object ) )
	{
		// If the object happens to be derived from TSharedFromThis, the following method
		// will prime the object with a weak pointer to itself.
		SharedPointerInternals::EnableSharedFromThis(SharedReferenceCount, InRawPtrProxy.Object);
	}

	/**
	 * Constructs a shared pointer using a proxy reference to a raw pointer. (See MakeShareable())
	 *
	 * @param  InRawPtrProxy  Proxy raw pointer that contains the object that the new shared pointer will reference
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <
		typename OtherType,
		typename DeleterType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TSharedPtr( SharedPointerInternals::TRawPtrProxyWithDeleter< OtherType, DeleterType > const& InRawPtrProxy )
		: Object( InRawPtrProxy.Object )
		, SharedReferenceCount( SharedPointerInternals::NewCustomReferenceController< Mode >( InRawPtrProxy.Object, InRawPtrProxy.Deleter ) )
	{
		// If the object happens to be derived from TSharedFromThis, the following method
		// will prime the object with a weak pointer to itself.
		SharedPointerInternals::EnableSharedFromThis(SharedReferenceCount, InRawPtrProxy.Object);
	}

	/**
	 * Constructs a shared pointer using a proxy reference to a raw pointer. (See MakeShareable())
	 *
	 * @param  InRawPtrProxy  Proxy raw pointer that contains the object that the new shared pointer will reference
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <
		typename OtherType,
		typename DeleterType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TSharedPtr( SharedPointerInternals::TRawPtrProxyWithDeleter< OtherType, DeleterType >&& InRawPtrProxy )
		: Object( InRawPtrProxy.Object )
		, SharedReferenceCount( SharedPointerInternals::NewCustomReferenceController< Mode >( InRawPtrProxy.Object, MoveTemp( InRawPtrProxy.Deleter ) ) )
	{
		// If the object happens to be derived from TSharedFromThis, the following method
		// will prime the object with a weak pointer to itself.
		SharedPointerInternals::EnableSharedFromThis(SharedReferenceCount, InRawPtrProxy.Object);
	}

	/**
	 * Constructs a shared pointer as a shared reference to an existing shared pointer's object.
	 * This constructor is needed so that we can implicitly upcast to base classes.
	 *
	 * @param  InSharedPtr  The shared pointer whose object we should create an additional reference to
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TSharedPtr( TSharedPtr< OtherType, Mode > const& InSharedPtr )
		: Object( InSharedPtr.Object )
		, SharedReferenceCount( InSharedPtr.SharedReferenceCount )
	{
	}

	UE_NODEBUG inline TSharedPtr( TSharedPtr const& InSharedPtr )
		: Object( InSharedPtr.Object )
		, SharedReferenceCount( InSharedPtr.SharedReferenceCount )
	{
	}

	UE_NODEBUG inline TSharedPtr( TSharedPtr&& InSharedPtr )
		: Object( InSharedPtr.Object )
		, SharedReferenceCount( MoveTemp(InSharedPtr.SharedReferenceCount) )
	{
		InSharedPtr.Object = nullptr;
	}

	/**
	 * Implicitly converts a shared reference to a shared pointer, adding a reference to the object.
	 * NOTE: We allow an implicit conversion from TSharedRef to TSharedPtr because it's always a safe conversion.
	 *
	 * @param  InSharedRef  The shared reference that will be converted to a shared pointer
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TSharedPtr( TSharedRef< OtherType, Mode > const& InSharedRef )
		: Object( InSharedRef.Object )
		, SharedReferenceCount( InSharedRef.SharedReferenceCount )
	{
		// There is no rvalue overload of this constructor, because 'stealing' the pointer from a
		// TSharedRef would leave it as null, which would invalidate its invariant.
	}

	/**
	 * Special constructor used internally to statically cast one shared pointer type to another.  You
	 * should never call this constructor directly.  Instead, use the StaticCastSharedPtr() function.
	 * This constructor creates a shared pointer as a shared reference to an existing shared pointer after
	 * statically casting that pointer's object.  This constructor is needed for static casts.
	 *
	 * @param  InSharedPtr  The shared pointer whose object we should create an additional reference to
	 */
	template <typename OtherType>
	UE_NODEBUG inline TSharedPtr( TSharedPtr< OtherType, Mode > const& InSharedPtr, SharedPointerInternals::FStaticCastTag )
		: Object( static_cast< ObjectType* >( InSharedPtr.Object ) )
		, SharedReferenceCount( InSharedPtr.SharedReferenceCount )
	{
	}
	
	template <typename OtherType>
	UE_NODEBUG TSharedPtr(TSharedPtr<OtherType, Mode>&& InSharedPtr, SharedPointerInternals::FStaticCastTag)
		: Object{static_cast<ObjectType*>(InSharedPtr.Object)}
		, SharedReferenceCount{MoveTemp(InSharedPtr.SharedReferenceCount)}
	{
		InSharedPtr.Object = nullptr;
	}
	
	/**
	 * Special constructor used internally to cast a 'const' shared pointer a 'mutable' pointer.  You
	 * should never call this constructor directly.  Instead, use the ConstCastSharedPtr() function.
	 * This constructor creates a shared pointer as a shared reference to an existing shared pointer after
	 * const casting that pointer's object.  This constructor is needed for const casts.
	 *
	 * @param  InSharedPtr  The shared pointer whose object we should create an additional reference to
	 */
	template <typename OtherType>
	UE_NODEBUG inline TSharedPtr( TSharedPtr< OtherType, Mode > const& InSharedPtr, SharedPointerInternals::FConstCastTag )
		: Object( const_cast< ObjectType* >( InSharedPtr.Object ) )
		, SharedReferenceCount( InSharedPtr.SharedReferenceCount )
	{
	}

	template <typename OtherType>
	UE_NODEBUG TSharedPtr(TSharedPtr<OtherType, Mode>&& InSharedPtr, SharedPointerInternals::FConstCastTag)
		: Object{const_cast<ObjectType*>(InSharedPtr.Object)}
		, SharedReferenceCount{MoveTemp(InSharedPtr.SharedReferenceCount)}
	{
		InSharedPtr.Object = nullptr;
	}
	
	/**
	 * Aliasing constructor used to create a shared pointer which shares its reference count with
	 * another shared object, but pointing to a different object, typically a subobject.
	 *
	 * @param  OtherSharedPtr  The shared pointer whose reference count should be shared.
	 * @param  InObject  The object pointer to use (instead of the incoming shared pointer's object)
	 */
	template <typename OtherType>
	UE_NODEBUG inline TSharedPtr( TSharedPtr< OtherType, Mode > const& OtherSharedPtr, ObjectType* InObject )
		: Object( InObject )
		, SharedReferenceCount( OtherSharedPtr.SharedReferenceCount )
	{
	}

	/**
	 * Aliasing constructor used to create a shared pointer which shares its reference count with
	 * another shared object, but pointing to a different object, typically a subobject.
	 *
	 * @param  OtherSharedPtr  The shared pointer whose reference count should be shared.
	 * @param  InObject  The object pointer to use (instead of the incoming shared pointer's object)
	 */
	template <typename OtherType>
	UE_NODEBUG inline TSharedPtr( TSharedPtr< OtherType, Mode >&& OtherSharedPtr, ObjectType* InObject )
		: Object( InObject )
		, SharedReferenceCount( MoveTemp(OtherSharedPtr.SharedReferenceCount) )
	{
		OtherSharedPtr.Object = nullptr;
	}

	/**
	 * Aliasing constructor used to create a shared pointer which shares its reference count with
	 * another shared object, but pointing to a different object, typically a subobject.
	 *
	 * @param  OtherSharedRef  The shared reference whose reference count should be shared.
	 * @param  InObject  The object pointer to use (instead of the incoming shared pointer's object)
	 */
	template <typename OtherType>
	UE_NODEBUG inline TSharedPtr( TSharedRef< OtherType, Mode > const& OtherSharedRef, ObjectType* InObject )
		: Object( InObject )
		, SharedReferenceCount( OtherSharedRef.SharedReferenceCount )
	{
	}

	/**
	 * Assignment to a nullptr pointer.  The object currently referenced by this shared pointer will no longer be
	 * referenced and will be deleted if there are no other referencers.
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	UE_REWRITE TSharedPtr& operator=( SharedPointerInternals::FNullTag* )
	{
		Reset();
		return *this;
	}

	/**
	 * Assignment operator replaces this shared pointer with the specified shared pointer.  The object
	 * currently referenced by this shared pointer will no longer be referenced and will be deleted if
	 * there are no other referencers.
	 *
	 * @param  InSharedPtr  Shared pointer to replace with
	 */
	UE_NODEBUG inline TSharedPtr& operator=( TSharedPtr const& InSharedPtr )
	{
		TSharedPtr Temp = InSharedPtr;
		::Swap(Temp, *this);
		return *this;
	}

	// Disable false positive buffer overrun warning during pgo linking step
	PGO_LINK_DISABLE_WARNINGS
	UE_NODEBUG inline TSharedPtr& operator=( TSharedPtr&& InSharedPtr )
	{
		if (this != &InSharedPtr)
		{
			Object = InSharedPtr.Object;
			InSharedPtr.Object = nullptr;
			SharedReferenceCount = MoveTemp(InSharedPtr.SharedReferenceCount);
		}
		return *this;
	}
	PGO_LINK_ENABLE_WARNINGS

	/**
	 * Assignment operator replaces this shared pointer with the specified shared pointer.  The object
	 * currently referenced by this shared pointer will no longer be referenced and will be deleted if
	 * there are no other referencers.
	 *
	 * @param  InRawPtrProxy  Proxy object used to assign the object (see MakeShareable helper function)
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_REWRITE TSharedPtr& operator=( SharedPointerInternals::TRawPtrProxy< OtherType > const& InRawPtrProxy )
	{
		*this = TSharedPtr< ObjectType, Mode >( InRawPtrProxy );
		return *this;
	}

	/**
	 * Assignment operator replaces this shared pointer with the specified shared pointer.  The object
	 * currently referenced by this shared pointer will no longer be referenced and will be deleted if
	 * there are no other referencers.
	 *
	 * @param  InRawPtrProxy  Proxy object used to assign the object (see MakeShareable helper function)
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <
		typename OtherType,
		typename DeleterType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_REWRITE TSharedPtr& operator=( SharedPointerInternals::TRawPtrProxyWithDeleter< OtherType, DeleterType > const& InRawPtrProxy )
	{
		*this = TSharedPtr< ObjectType, Mode >( InRawPtrProxy );
		return *this;
	}

	/**
	 * Assignment operator replaces this shared pointer with the specified shared pointer.  The object
	 * currently referenced by this shared pointer will no longer be referenced and will be deleted if
	 * there are no other referencers.
	 *
	 * @param  InRawPtrProxy  Proxy object used to assign the object (see MakeShareable helper function)
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <
		typename OtherType,
		typename DeleterType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_REWRITE TSharedPtr& operator=( SharedPointerInternals::TRawPtrProxyWithDeleter< OtherType, DeleterType >&& InRawPtrProxy )
	{
		*this = TSharedPtr< ObjectType, Mode >( MoveTemp( InRawPtrProxy ) );
		return *this;
	}

	/** TSharedPtr can be returned from AutoRTFM::Open() via a copy-construction in the closed. */
	static constexpr AutoRTFM::EReturnFromOpenMode AutoRTFMReturnFromOpenMode = AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed;

	/**
	 * Converts a shared pointer to a shared reference.  The pointer *must* be valid or an assertion will trigger.
	 *
	 * @return  Reference to the object
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	[[nodiscard]] UE_NODEBUG inline TSharedRef< ObjectType, Mode > ToSharedRef() const&
	{
		// If this assert goes off, it means a shared reference was created from a shared pointer that was nullptr.
		// Shared references are never allowed to be null.  Consider using TSharedPtr instead.
		check( IsValid() );
		return TSharedRef< ObjectType, Mode >( *this );
	}

	/**
	 * Converts a shared pointer to a shared reference.  The pointer *must* be valid or an assertion will trigger.
	 *
	 * @return  Reference to the object
	 */
	 // NOTE: The following is an Unreal extension to standard shared_ptr behavior
	[[nodiscard]] UE_NODEBUG inline TSharedRef< ObjectType, Mode > ToSharedRef() &&
	{
		// If this assert goes off, it means a shared reference was created from a shared pointer that was nullptr.
		// Shared references are never allowed to be null.  Consider using TSharedPtr instead.
		check( IsValid() );
		return TSharedRef< ObjectType, Mode >( MoveTemp( *this ) );
	}

	/**
	 * Converts a shared pointer to a weak ptr.
	 *
	 * @return  Weak pointer to the object
	 */
	[[nodiscard]] UE_REWRITE TWeakPtr<ObjectType, Mode> ToWeakPtr() const
	{
		return TWeakPtr<ObjectType, Mode>(*this);
	}

	/**
	 * Returns the object referenced by this pointer, or nullptr if no object is reference
	 *
	 * @return  The object owned by this shared pointer, or nullptr
	 */
	[[nodiscard]] UE_REWRITE ObjectType* Get() const
	{
		return Object;
	}

	/**
	 * Checks to see if this shared pointer is actually pointing to an object
	 *
	 * @return  True if the shared pointer is valid and can be dereferenced
	 */
	[[nodiscard]] UE_REWRITE explicit operator bool() const
	{
		return Object != nullptr;
	}

	/**
	 * Checks to see if this shared pointer is actually pointing to an object
	 *
	 * @return  True if the shared pointer is valid and can be dereferenced
	 */
	[[nodiscard]] UE_REWRITE const bool IsValid() const
	{
		return Object != nullptr;
	}

	/**
	 * Dereference operator returns a reference to the object this shared pointer points to
	 *
	 * @return  Reference to the object
	 */
	[[nodiscard]] UE_NODEBUG inline std::add_lvalue_reference_t<ObjectType> operator*() const
		requires requires (ObjectType* Ptr) { *Ptr; }
	{
		check( IsValid() );
		return *Object;
	}

	/**
	 * Arrow operator returns a pointer to the object this shared pointer references
	 *
	 * @return  Returns a pointer to the object referenced by this shared pointer
	 */
	[[nodiscard]] UE_NODEBUG inline ObjectType* operator->() const
	{
		check( IsValid() );
		return Object;
	}

	/**
	 * Resets this shared pointer, removing a reference to the object.  If there are no other shared
	 * references to the object then it will be destroyed.
	 */
	UE_REWRITE void Reset()
	{
 		*this = TSharedPtr< ObjectType, Mode >();
	}

	/**
	 * Returns the number of shared references to this object (including this reference.)
	 * IMPORTANT: Not necessarily fast!  Should only be used for debugging purposes!
	 *
	 * @return  Number of shared references to the object (including this reference.)
	 */
	[[nodiscard]] UE_REWRITE int32 GetSharedReferenceCount() const
	{
		return SharedReferenceCount.GetSharedReferenceCount();
	}

	/**
	 * Returns true if this is the only shared reference to this object.  Note that there may be
	 * outstanding weak references left.
	 * IMPORTANT: Not necessarily fast!  Should only be used for debugging purposes!
	 *
	 * @return  True if there is only one shared reference to the object, and this is it!
	 */
	[[nodiscard]] UE_REWRITE bool IsUnique() const
	{
		return SharedReferenceCount.IsUnique();
	}

	///////////////////////////////////////////////////
	// Start - intrusive TOptional<TSharedPtr> state //
	///////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TSharedPtr;

	UE_REWRITE explicit TSharedPtr(FIntrusiveUnsetOptionalState)
		: Object((ObjectType*)-1)
	{
	}
	UE_REWRITE bool UEOpEquals(FIntrusiveUnsetOptionalState) const
	{
		return Object == (ObjectType*)-1;
	}
	/////////////////////////////////////////////////
	// End - intrusive TOptional<TSharedPtr> state //
	/////////////////////////////////////////////////

	template <typename OtherType>
	[[nodiscard]] UE_REWRITE bool UEOpEquals(const TSharedPtr<OtherType, Mode>& Rhs) const
	{
		return this->Object == Rhs.Get();
	}

	template <typename OtherType>
	[[nodiscard]] UE_REWRITE bool UEOpEquals(const TWeakPtr<OtherType, Mode>& Rhs) const
	{
		return this->Object == Rhs.Pin().Get();
	}

	[[nodiscard]] UE_REWRITE bool UEOpEquals(TYPE_OF_NULLPTR) const
	{
		return !this->Object;
	}

	template <typename OtherType>
	[[nodiscard]] UE_REWRITE bool UEOpLessThan(const TSharedPtr<OtherType, Mode>& Rhs) const
	{
		return this->Object < Rhs.Get();
	}

private:
	/**
	 * Constructs a shared pointer from a weak pointer, allowing you to access the object (if it
	 * hasn't expired yet.)  Remember, if there are no more shared references to the object, the
	 * shared pointer will not be valid.  You should always check to make sure this shared
	 * pointer is valid before trying to dereference the shared pointer!
	 *
	 * NOTE: This constructor is private to force users to be explicit when converting a weak
	 *       pointer to a shared pointer.  Use the weak pointer's Pin() method instead!
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline explicit TSharedPtr( TWeakPtr< OtherType, Mode > const& InWeakPtr )
		: Object( nullptr )
		, SharedReferenceCount( InWeakPtr.WeakReferenceCount )
	{
		// Check that the shared reference was created from the weak reference successfully.  We'll only
		// cache a pointer to the object if we have a valid shared reference.
		if( SharedReferenceCount.IsValid() )
		{
			Object = InWeakPtr.Object;
		}
	}

	/**
	 * Constructs a shared pointer from a weak pointer, allowing you to access the object (if it
	 * hasn't expired yet.)  Remember, if there are no more shared references to the object, the
	 * shared pointer will not be valid.  You should always check to make sure this shared
	 * pointer is valid before trying to dereference the shared pointer!
	 *
	 * NOTE: This constructor is private to force users to be explicit when converting a weak
	 *       pointer to a shared pointer.  Use the weak pointer's Pin() method instead!
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline explicit TSharedPtr(TWeakPtr< OtherType, Mode >&& InWeakPtr)
		: Object(nullptr)
		, SharedReferenceCount( MoveTemp( InWeakPtr.WeakReferenceCount ) )
	{
		// Check that the shared reference was created from the weak reference successfully.  We'll only
		// cache a pointer to the object if we have a valid shared reference.
		if (SharedReferenceCount.IsValid())
		{
			Object = InWeakPtr.Object;
			InWeakPtr.Object = nullptr;
		}
	}

	/**
	 * Aliasing constructor used to create a shared pointer which shares its reference count with
	 * another shared object, but pointing to a different object, typically a subobject.
	 *
	 * @param  InSharedReferenceCount  The shared referencer whose reference count should be shared.
	 * @param  InObject                The object pointer to use
	 */
	UE_NODEBUG TSharedPtr(SharedPointerInternals::FSharedReferencer<Mode> InSharedReferenceCount, ObjectType* InObject)
		: Object{InObject}
		, SharedReferenceCount{MoveTemp(InSharedReferenceCount)}
	{
	}

	// We declare ourselves as a friend (templated using OtherType) so we can access members as needed
	template< class OtherType, ESPMode OtherMode > friend class TSharedPtr;

	// Declare other smart pointer types as friends as needed
	template< class OtherType, ESPMode OtherMode > friend class TSharedRef;
	template< class OtherType, ESPMode OtherMode > friend class TWeakPtr;
	template< class OtherType, ESPMode OtherMode > friend class TSharedFromThis;

private:

	/** The object we're holding a reference to.  Can be nullptr. */
	ObjectType* Object;

	/** Interface to the reference counter for this object.  Note that the actual reference
		controller object is shared by all shared and weak pointers that refer to the object */
	SharedPointerInternals::FSharedReferencer< Mode > SharedReferenceCount;
};

/**
 * Trait which determines whether or not a type is a TSharedPtr.
 */
template <typename T>                       constexpr bool TIsTSharedPtr_V                                                = false;
template <class ObjectType, ESPMode InMode> constexpr bool TIsTSharedPtr_V<               TSharedPtr<ObjectType, InMode>> = true;
template <class ObjectType, ESPMode InMode> constexpr bool TIsTSharedPtr_V<const          TSharedPtr<ObjectType, InMode>> = true;
template <class ObjectType, ESPMode InMode> constexpr bool TIsTSharedPtr_V<      volatile TSharedPtr<ObjectType, InMode>> = true;
template <class ObjectType, ESPMode InMode> constexpr bool TIsTSharedPtr_V<const volatile TSharedPtr<ObjectType, InMode>> = true;

namespace Freeze
{
	template<class ObjectType, ESPMode Mode>
	void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const TSharedPtr<ObjectType, Mode>& Object, const FTypeLayoutDesc&)
	{
		// we never want to freeze pointers, so write an empty one
		Writer.WriteBytes(TSharedPtr<ObjectType, Mode>());
	}
}

DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT((template<class ObjectType, ESPMode Mode>), (TSharedPtr<ObjectType, Mode>));

template<class ObjectType, ESPMode Mode> struct TIsZeroConstructType<TSharedPtr<ObjectType, Mode>> { enum { Value = true }; };

template <class ObjectType, ESPMode Mode>
struct TCallTraits<TSharedPtr<ObjectType, Mode>> : public TCallTraitsBase<TSharedPtr<ObjectType, Mode>>
{
	using ConstPointerType = TSharedPtr<const ObjectType, Mode>;
};


/**
 * TWeakPtr is a non-intrusive reference-counted weak object pointer.  This weak pointer will be
 * conditionally thread-safe when the optional Mode template argument is set to ThreadSafe.
 */
template< class ObjectType, ESPMode InMode >
class TWeakPtr
{
public:
	using ElementType = ObjectType;
	static constexpr ESPMode Mode = InMode;

	/** Constructs an empty TWeakPtr */
	// NOTE: FNullTag parameter is an Unreal extension to standard shared_ptr behavior
	UE_NODEBUG inline TWeakPtr( SharedPointerInternals::FNullTag* = nullptr )
		: Object( nullptr )
		, WeakReferenceCount()
	{
	}

	/**
	 * Constructs a weak pointer from a shared reference
	 *
	 * @param  InSharedRef  The shared reference to create a weak pointer from
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TWeakPtr( TSharedRef< OtherType, Mode > const& InSharedRef )
		: Object( InSharedRef.Object )
		, WeakReferenceCount( InSharedRef.SharedReferenceCount )
	{
	}

	/**
	 * Constructs a weak pointer from a shared pointer
	 *
	 * @param  InSharedPtr  The shared pointer to create a weak pointer from
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TWeakPtr( TSharedPtr< OtherType, Mode > const& InSharedPtr )
		: Object( InSharedPtr.Object )
		, WeakReferenceCount( InSharedPtr.SharedReferenceCount )
	{
	}

	/**
	 * Special constructor used internally to statically cast one weak pointer type to another.  You
	 * should never call this constructor directly.  Instead, use the StaticCastWeakPtr() function.
	 * This constructor creates a weak pointer as a weak reference to an existing weak pointer after
	 * statically casting that pointer's object.  This constructor is needed for static casts.
	 *
	 * @param  InWeakPtr  The weak pointer whose object we should create an additional reference to
	 */
	template <typename OtherType>
	UE_NODEBUG inline TWeakPtr(TWeakPtr< OtherType, Mode > const& InWeakPtr, SharedPointerInternals::FStaticCastTag)
		: Object(static_cast<ObjectType*>(InWeakPtr.Object))
		, WeakReferenceCount(InWeakPtr.WeakReferenceCount)
	{
	}

	template <typename OtherType>
	UE_NODEBUG TWeakPtr(TWeakPtr<OtherType, Mode>&& InWeakPtr, SharedPointerInternals::FStaticCastTag)
		: Object{static_cast<ObjectType*>(InWeakPtr.Object)}
		, WeakReferenceCount{MoveTemp(InWeakPtr.WeakReferenceCount)}
	{
		InWeakPtr.Object = nullptr;
	}

	/**
	 * Special constructor used internally to cast a 'const' weak pointer a 'mutable' pointer.  You
	 * should never call this constructor directly.  Instead, use the ConstCastWeakPtr() function.
	 * This constructor creates a weak pointer as a weak reference to an existing weak pointer after
	 * const casting that pointer's object.  This constructor is needed for const casts.
	 *
	 * @param  InWeakPtr  The weak pointer whose object we should create an additional reference to
	 */
	template <typename OtherType>
	UE_NODEBUG inline TWeakPtr(TWeakPtr< OtherType, Mode > const& InWeakPtr, SharedPointerInternals::FConstCastTag)
		: Object(const_cast<ObjectType*>(InWeakPtr.Object))
		, WeakReferenceCount(InWeakPtr.WeakReferenceCount)
	{
	}

	template <typename OtherType>
	UE_NODEBUG TWeakPtr(TWeakPtr<OtherType, Mode>&& InWeakPtr, SharedPointerInternals::FConstCastTag)
		: Object{const_cast<ObjectType*>(InWeakPtr.Object)}
		, WeakReferenceCount{MoveTemp(InWeakPtr.WeakReferenceCount)}
	{
		InWeakPtr.Object = nullptr;
	}

	/**
	 * Constructs a weak pointer from a weak pointer of another type.
	 * This constructor is intended to allow derived-to-base conversions.
	 *
	 * @param  InWeakPtr  The weak pointer to create a weak pointer from
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TWeakPtr( TWeakPtr< OtherType, Mode > const& InWeakPtr )
		: Object( InWeakPtr.Object )
		, WeakReferenceCount( InWeakPtr.WeakReferenceCount )
	{
	}

	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TWeakPtr( TWeakPtr< OtherType, Mode >&& InWeakPtr )
		: Object( InWeakPtr.Object )
		, WeakReferenceCount( MoveTemp(InWeakPtr.WeakReferenceCount) )
	{
		InWeakPtr.Object = nullptr;
	}

	UE_NODEBUG inline TWeakPtr( TWeakPtr const& InWeakPtr )
		: Object( InWeakPtr.Object )
		, WeakReferenceCount( InWeakPtr.WeakReferenceCount )
	{
	}

	UE_NODEBUG inline TWeakPtr( TWeakPtr&& InWeakPtr )
		: Object( InWeakPtr.Object )
		, WeakReferenceCount( MoveTemp(InWeakPtr.WeakReferenceCount) )
	{
		InWeakPtr.Object = nullptr;
	}

	/**
	 * Assignment to a nullptr pointer.  Clears this weak pointer's reference.
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	UE_REWRITE TWeakPtr& operator=( SharedPointerInternals::FNullTag* )
	{
		Reset();
		return *this;
	}

	/**
	 * Assignment operator adds a weak reference to the object referenced by the specified weak pointer
	 *
	 * @param  InWeakPtr  The weak pointer for the object to assign
	 */
	UE_NODEBUG inline TWeakPtr& operator=( TWeakPtr const& InWeakPtr )
	{
		TWeakPtr Temp = InWeakPtr;
		::Swap(Temp, *this);
		return *this;
	}

	UE_NODEBUG inline TWeakPtr& operator=( TWeakPtr&& InWeakPtr )
	{
		if (this != &InWeakPtr)
		{
			Object             = InWeakPtr.Object;
			InWeakPtr.Object   = nullptr;
			WeakReferenceCount = MoveTemp(InWeakPtr.WeakReferenceCount);
		}
		return *this;
	}

	/**
	 * Assignment operator adds a weak reference to the object referenced by the specified weak pointer.
	 * This assignment operator is intended to allow derived-to-base conversions.
	 *
	 * @param  InWeakPtr  The weak pointer for the object to assign
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TWeakPtr& operator=( TWeakPtr<OtherType, Mode> const& InWeakPtr )
	{
		Object = InWeakPtr.Pin().Get();
		WeakReferenceCount = InWeakPtr.WeakReferenceCount;
		return *this;
	}

	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TWeakPtr& operator=( TWeakPtr<OtherType, Mode>&& InWeakPtr )
	{
		Object             = InWeakPtr.Object;
		InWeakPtr.Object   = nullptr;
		WeakReferenceCount = MoveTemp(InWeakPtr.WeakReferenceCount);
		return *this;
	}

	/**
	 * Assignment operator sets this weak pointer from a shared reference
	 *
	 * @param  InSharedRef  The shared reference used to assign to this weak pointer
	 */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TWeakPtr& operator=( TSharedRef< OtherType, Mode > const& InSharedRef )
	{
		Object = InSharedRef.Object;
		WeakReferenceCount = InSharedRef.SharedReferenceCount;
		return *this;
	}

	/**
	 * Assignment operator sets this weak pointer from a shared pointer
	 *
	 * @param  InSharedPtr  The shared pointer used to assign to this weak pointer
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	UE_NODEBUG inline TWeakPtr& operator=( TSharedPtr< OtherType, Mode > const& InSharedPtr )
	{
		Object = InSharedPtr.Object;
		WeakReferenceCount = InSharedPtr.SharedReferenceCount;
		return *this;
	}

	/** TWeakPtr can be returned from AutoRTFM::Open() via a copy-construction in the closed. */
	static constexpr AutoRTFM::EReturnFromOpenMode AutoRTFMReturnFromOpenMode = AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed;

	/**
	 * Converts this weak pointer to a shared pointer that you can use to access the object (if it
	 * hasn't expired yet.)  Remember, if there are no more shared references to the object, the
	 * returned shared pointer will not be valid.  You should always check to make sure the returned
	 * pointer is valid before trying to dereference the shared pointer!
	 *
	 * @return  Shared pointer for this object (will only be valid if still referenced!)
	 */
	[[nodiscard]] UE_REWRITE TSharedPtr< ObjectType, Mode > Pin() const&
	{
		return TSharedPtr< ObjectType, Mode >( *this );
	}

	/**
	 * Converts this weak pointer to a shared pointer that you can use to access the object (if it
	 * hasn't expired yet.)  Remember, if there are no more shared references to the object, the
	 * returned shared pointer will not be valid.  You should always check to make sure the returned
	 * pointer is valid before trying to dereference the shared pointer!
	 *
	 * @return  Shared pointer for this object (will only be valid if still referenced!)
	 */
	[[nodiscard]] UE_REWRITE TSharedPtr< ObjectType, Mode > Pin() &&
	{
		return TSharedPtr< ObjectType, Mode >( MoveTemp( *this ) );
	}

	/**
	 * Checks to see if this weak pointer actually has a valid reference to an object
	 *
	 * @return  True if the weak pointer is valid and a pin operator would have succeeded
	 */
	[[nodiscard]] UE_NODEBUG inline bool IsValid() const
	{
		return Object != nullptr && WeakReferenceCount.IsValid();
	}

	/**
	 * Resets this weak pointer, removing a weak reference to the object.  If there are no other shared
	 * or weak references to the object, then the tracking object will be destroyed.
	 */
	UE_REWRITE void Reset()
	{
		*this = TWeakPtr< ObjectType, Mode >();
	}

	/**
	 * Returns true if the object this weak pointer points to is the same as the specified object pointer.
	 */
	[[nodiscard]] UE_REWRITE bool HasSameObject( const void* InOtherPtr ) const
	{
		return Pin().Get() == InOtherPtr;
	}

	UE_REWRITE uint32 GetWeakPtrTypeHash() const
	{
		return ::PointerHash( Object );
	}

	/////////////////////////////////////////////////
	// Start - intrusive TOptional<TWeakPtr> state //
	/////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TWeakPtr;

	UE_REWRITE explicit TWeakPtr(FIntrusiveUnsetOptionalState)
		: Object((ObjectType*)-1)
	{
	}
	UE_REWRITE bool UEOpEquals(FIntrusiveUnsetOptionalState) const
	{
		return Object == (ObjectType*)-1;
	}
	///////////////////////////////////////////////
	// End - intrusive TOptional<TWeakPtr> state //
	///////////////////////////////////////////////

	template <typename OtherType>
	[[nodiscard]] UE_REWRITE bool UEOpEquals(const TWeakPtr<OtherType, Mode>& Rhs) const
	{
		return this->Pin().Get() == Rhs.Pin().Get();
	}

	[[nodiscard]] UE_REWRITE bool UEOpEquals(TYPE_OF_NULLPTR) const
	{
		return !this->IsValid();
	}

	template <typename OtherType>
	[[nodiscard]] UE_REWRITE bool UEOpLessThan(const TWeakPtr<OtherType, Mode>& Rhs) const
	{
		return this->Pin().Get() < Rhs.Pin().Get();
	}

private:
	/**
	 * Aliasing constructor used to create a weak pointer which shares its reference count with
	 * another shared object, but pointing to a different object, typically a subobject.
	 *
	 * @param  InWeakReferenceCount  The weak referencer whose reference count should be shared.
	 * @param  InObject              The object pointer to use
	 */
	UE_NODEBUG TWeakPtr(SharedPointerInternals::FWeakReferencer<Mode> InWeakReferenceCount, ObjectType* InObject)
		: Object{InObject}
		, WeakReferenceCount{MoveTemp(InWeakReferenceCount)}
	{
	}

	// We declare ourselves as a friend (templated using OtherType) so we can access members as needed
    template< class OtherType, ESPMode OtherMode > friend class TWeakPtr;

	// Declare ourselves as a friend of TSharedPtr so we can access members as needed
    template< class OtherType, ESPMode OtherMode > friend class TSharedPtr;
	template <class OtherType, ESPMode OtherMode> friend class TSharedFromThis;

private:
	/** The object we have a weak reference to.  Can be nullptr.  Also, it's important to note that because
	    this is a weak reference, the object this pointer points to may have already been destroyed. */
	ObjectType* Object;

	/** Interface to the reference counter for this object.  Note that the actual reference
		controller object is shared by all shared and weak pointers that refer to the object */
	SharedPointerInternals::FWeakReferencer< Mode > WeakReferenceCount;
};

/**
 * Trait which determines whether or not a type is a TWeakPtr.
 */
template <typename T>                       constexpr bool TIsTWeakPtr_V                                              = false;
template <class ObjectType, ESPMode InMode> constexpr bool TIsTWeakPtr_V<               TWeakPtr<ObjectType, InMode>> = true;
template <class ObjectType, ESPMode InMode> constexpr bool TIsTWeakPtr_V<const          TWeakPtr<ObjectType, InMode>> = true;
template <class ObjectType, ESPMode InMode> constexpr bool TIsTWeakPtr_V<      volatile TWeakPtr<ObjectType, InMode>> = true;
template <class ObjectType, ESPMode InMode> constexpr bool TIsTWeakPtr_V<const volatile TWeakPtr<ObjectType, InMode>> = true;


template<class T, ESPMode Mode> struct TIsWeakPointerType<TWeakPtr<T, Mode> > { enum { Value = true }; };
template<class T, ESPMode Mode> struct TIsZeroConstructType<TWeakPtr<T, Mode> > { enum { Value = true }; };

template <class ObjectType, ESPMode Mode>
struct TCallTraits<TWeakPtr<ObjectType, Mode>> : public TCallTraitsBase<TWeakPtr<ObjectType, Mode>>
{
	using ConstPointerType = TWeakPtr<const ObjectType, Mode>;
};


/**
 * Derive your class from TSharedFromThis to enable access to a TSharedRef directly from an object
 * instance that's already been allocated.  Use the optional Mode template argument for thread-safety.
 */
template< class ObjectType, ESPMode Mode >
class TSharedFromThis : private UE::Core::Private::FSharedFromThisBase
{
public:

	/**
	 * Provides a shared reference to this object.  Note that is only valid to call
	 * this after a shared reference (or shared pointer) to the object has already been created.
	 * Also note that it is illegal to call this in the object's destructor.
	 *
	 * @return	Returns this object as a shared reference
	 */
	[[nodiscard]] UE_NODEBUG TSharedRef< ObjectType, Mode > AsShared()
	{
		//
		// If the following assert goes off, it means one of the following:
		//
		//     - You tried to request a shared reference before the object was ever assigned to one. (e.g. constructor)
		//     - You tried to request a shared reference while the object is being destroyed (destructor chain)
		//
		// To fix this, make sure you create at least one shared reference to your object instance before requested,
		// and also avoid calling this function from your object's destructor.
		//
		check(DoesSharedInstanceExist());
		
		ReferenceController->AddSharedReference();
		TSharedPtr<ObjectType, Mode> SharedThis{SharedPointerInternals::FSharedReferencer<Mode>{ReferenceController}, static_cast<ObjectType*>(this)};

		// Now that we've verified the shared pointer is valid, we'll convert it to a shared reference
		// and return it!
		return MoveTemp( SharedThis ).ToSharedRef();
	}

	/**
	 * Provides a shared reference to this object (const.)  Note that is only valid to call
	 * this after a shared reference (or shared pointer) to the object has already been created.
	 * Also note that it is illegal to call this in the object's destructor.
	 *
	 * @return	Returns this object as a shared reference (const)
	 */
	[[nodiscard]] UE_NODEBUG TSharedRef< ObjectType const, Mode > AsShared() const
	{
		//
		// If the following assert goes off, it means one of the following:
		//
		//     - You tried to request a shared reference before the object was ever assigned to one. (e.g. constructor)
		//     - You tried to request a shared reference while the object is being destroyed (destructor chain)
		//
		// To fix this, make sure you create at least one shared reference to your object instance before requested,
		// and also avoid calling this function from your object's destructor.
		//
		check(DoesSharedInstanceExist());
		
		ReferenceController->AddSharedReference();
		TSharedPtr<ObjectType const, Mode> SharedThis{SharedPointerInternals::FSharedReferencer<Mode>{ReferenceController}, static_cast<ObjectType const*>(this)};

		// Now that we've verified the shared pointer is valid, we'll convert it to a shared reference
		// and return it!
		return MoveTemp( SharedThis ).ToSharedRef();
	}

	/**
	 * Provides a shared reference to a subobject of this object, i.e. points to an object which shares
	 * the lifetime and refcount of this object, but isn't derived from TSharedFromThis.  Note that is only valid to call
	 * this after a shared reference (or shared pointer) to the object has already been created.
	 * Also note that it is illegal to call this in the object's destructor.
	 *
	 * @return	Returns the subobject as a shared reference.
	 */
	template< typename SubobjectType >
	[[nodiscard]] UE_REWRITE TSharedRef< SubobjectType, Mode > AsSharedSubobject( SubobjectType* SubobjectPtr ) const
	{
		return TSharedRef< SubobjectType, Mode >( AsShared(), SubobjectPtr );
	}

	/**
	 * Provides a weak reference to this object.  Note that is only valid to call
	 * this after a shared reference (or shared pointer) to the object has already been created.
	 * Also note that it is illegal to call this in the object's destructor.
	 *
	 * @return	Returns this object as a weak pointer
	 */
	[[nodiscard]] UE_NODEBUG TWeakPtr< ObjectType, Mode > AsWeak()
	{
		//
		// If the following assert goes off, it means one of the following:
		//
		//     - You tried to request a weak pointer before the object was ever assigned to a shared pointer. (e.g. constructor)
		//     - You tried to request a weak pointer while the object is being destroyed (destructor chain)
		//
		// To fix this, make sure you create at least one shared reference to your object instance before requested,
		// and also avoid calling this function from your object's destructor.
		//
		check(DoesSharedInstanceExist());
		
		ReferenceController->AddWeakReference();
		TWeakPtr<ObjectType, Mode> Result{SharedPointerInternals::FWeakReferencer<Mode>{ReferenceController}, static_cast<ObjectType*>(this)};

		// Now that we've verified the pointer is valid, we'll return it!
		return Result;
	}
	[[nodiscard]] UE_NODEBUG TWeakPtr< ObjectType const, Mode > AsWeak() const
	{
		//
		// If the following assert goes off, it means one of the following:
		//
		//     - You tried to request a weak pointer before the object was ever assigned to a shared pointer. (e.g. constructor)
		//     - You tried to request a weak pointer while the object is being destroyed (destructor chain)
		//
		// To fix this, make sure you create at least one shared reference to your object instance before requested,
		// and also avoid calling this function from your object's destructor.
		//
		check(DoesSharedInstanceExist());
		
		ReferenceController->AddWeakReference();
		TWeakPtr<ObjectType const, Mode> Result{SharedPointerInternals::FWeakReferencer<Mode>{ReferenceController}, static_cast<ObjectType const*>(this)};

		// Now that we've verified the pointer is valid, we'll return it!
		return Result;
	}

	/**
	 * Provides a weak pointer to a subobject of this object, i.e. points to an object which shares
	 * the lifetime and refcount of this object, but isn't derived from TSharedFromThis.  Note that is only valid to call
	 * this after a shared reference (or shared pointer) to the object has already been created.
	 * Also note that it is illegal to call this in the object's destructor.
	 *
	 * @return	Returns the subobject as a weak pointer.
	 */
	template <typename SubobjectType>
	[[nodiscard]] UE_REWRITE TWeakPtr< SubobjectType, Mode > AsWeakSubobject( SubobjectType* SubobjectPtr ) const
	{
		return AsSharedSubobject( SubobjectPtr ).ToWeakPtr();
	}

protected:

	/**
	 * Provides access to a shared reference to an object, given the object's 'this' pointer.  Uses
	 * the 'this' pointer to derive the object's actual type, then casts and returns an appropriately
	 * typed shared reference.  Intentionally declared 'protected', as should only be called when the
	 * 'this' pointer can be passed.
	 *
	 * @return	Returns this object as a shared pointer
	 */
	template< class OtherType >
	[[nodiscard]] UE_REWRITE static TSharedRef< OtherType, Mode > SharedThis( OtherType* ThisPtr )
	{
		return StaticCastSharedRef< OtherType >( ThisPtr->AsShared() );
	}

	/**
	 * Provides access to a shared reference to an object, given the object's 'this' pointer. Uses
	 * the 'this' pointer to derive the object's actual type, then casts and returns an appropriately
	 * typed shared reference.  Intentionally declared 'protected', as should only be called when the
	 * 'this' pointer can be passed.
	 *
	 * @return	Returns this object as a shared pointer (const)
	 */
	template< class OtherType >
	[[nodiscard]] UE_REWRITE static TSharedRef< OtherType const, Mode > SharedThis( const OtherType* ThisPtr )
	{
		return StaticCastSharedRef< OtherType const >( ThisPtr->AsShared() );
	}

private:

	/**
	 * Freshens the internal weak pointer object using the authoritative shared reference to the object.
	 * Note that until this function is called, calls to AsShared() will result in an empty pointer.
	 */
	template <ESPMode OtherMode>
	UE_NODEBUG void UpdateWeakReferenceInternal(SharedPointerInternals::TReferenceControllerBase<OtherMode>* InReferenceController) const
	{
		static_assert(OtherMode == Mode, "You cannot create a shared reference or pointer of one mode with a type which inherits TSharedFromThis of another mode.");

		if (!DoesSharedInstanceExist() && ReferenceController != InReferenceController)
		{
			if (InReferenceController)
			{
				InReferenceController->AddWeakReference();
			}
			
			if (ReferenceController)
			{
				ReferenceController->ReleaseWeakReference();
			}
			
			ReferenceController = InReferenceController;
		}
	}

public:

	/**
	 * Checks whether our referenced instance is valid (ie, whether it's safe to call AsShared).
	 * If this returns false, it means that your instance has either:
	 *  - Not yet been assigned to a shared pointer (via MakeShared or MakeShareable).
	 *  - Is currently within its constructor (so the shared instance isn't yet available).
	 *  - Is currently within its destructor (so the shared instance is no longer available).
	 */
	[[nodiscard]] UE_REWRITE bool DoesSharedInstanceExist() const
	{
		return ReferenceController && ReferenceController->GetSharedReferenceCount() > 0;
	}

protected:

	/** Hidden stub constructor */
	TSharedFromThis()
	{
#ifndef UE_HEADER_UNITS
		// Cause a compile error if ObjectType is a UObject.
		int32 UObjectTestOverload(const volatile UObject*);
		int16 UObjectTestOverload(...);
		static_assert(sizeof(UObjectTestOverload((ObjectType*)nullptr)) == sizeof(int16), "TSharedFromThis is not supported on UObjects");
#endif
	}

	/** Hidden stub copy constructor */
	TSharedFromThis( TSharedFromThis const& ) { }

	/** Hidden stub assignment operator */
	UE_REWRITE TSharedFromThis& operator=( TSharedFromThis const& )
	{
		return *this;
	}

	/** Hidden destructor */
	~TSharedFromThis()
	{
		if (ReferenceController)
		{
			ReferenceController->ReleaseWeakReference();
		}
	}

private:
	
	template <class SharedReferencerType, class OtherType>
	friend void SharedPointerInternals::EnableSharedFromThis(SharedReferencerType& InSharedReferencer, OtherType const* InObject);

	/** Note this is declared mutable only so that UpdateWeakReferenceInternal() can update it. */
	mutable SharedPointerInternals::TReferenceControllerBase<Mode>* ReferenceController = nullptr;
};


/**
 * Casts a shared pointer of one type to another type. (static_cast)  Useful for down-casting.
 *
 * @param  InSharedPtr  The shared pointer to cast
 */
template< class CastToType, class CastFromType, ESPMode Mode >
[[nodiscard]] UE_REWRITE TSharedPtr< CastToType, Mode > StaticCastSharedPtr(TSharedPtr< CastFromType, Mode > const& InSharedPtr)
{
	return TSharedPtr< CastToType, Mode >(InSharedPtr, SharedPointerInternals::FStaticCastTag());
}

template <class CastToType, class CastFromType, ESPMode Mode>
[[nodiscard]] UE_REWRITE TSharedPtr<CastToType, Mode> StaticCastSharedPtr(TSharedPtr<CastFromType, Mode>&& InSharedPtr)
{
	return TSharedPtr<CastToType, Mode>{MoveTemp(InSharedPtr), SharedPointerInternals::FStaticCastTag{}};
}


/**
 * Casts a weak pointer of one type to another type. (static_cast)  Useful for down-casting.
 *
 * @param  InWeakPtr  The weak pointer to cast
 */
template< class CastToType, class CastFromType, ESPMode Mode >
[[nodiscard]] UE_REWRITE TWeakPtr< CastToType, Mode > StaticCastWeakPtr(TWeakPtr< CastFromType, Mode > const& InWeakPtr)
{
	return TWeakPtr< CastToType, Mode >(InWeakPtr, SharedPointerInternals::FStaticCastTag());
}

template <class CastToType, class CastFromType, ESPMode Mode>
[[nodiscard]] UE_REWRITE TWeakPtr<CastToType, Mode> StaticCastWeakPtr(TWeakPtr<CastFromType, Mode>&& InWeakPtr)
{
	return TWeakPtr<CastToType, Mode>{MoveTemp(InWeakPtr), SharedPointerInternals::FStaticCastTag{}};
}


/**
 * Casts a 'const' shared reference to 'mutable' shared reference. (const_cast)
 *
 * @param  InSharedRef  The shared reference to cast
 */
template< class CastToType, class CastFromType, ESPMode Mode >
[[nodiscard]] UE_REWRITE TSharedRef< CastToType, Mode > ConstCastSharedRef( TSharedRef< CastFromType, Mode > const& InSharedRef )
{
	return TSharedRef< CastToType, Mode >( InSharedRef, SharedPointerInternals::FConstCastTag() );
}

template <class CastToType, class CastFromType, ESPMode Mode>
[[nodiscard]] UE_REWRITE TSharedRef<CastToType, Mode> ConstCastSharedRef(TSharedRef<CastFromType, Mode>&& InSharedRef)
{
	return TSharedRef<CastToType, Mode>{MoveTemp(InSharedRef), SharedPointerInternals::FConstCastTag{}};
}


/**
 * Casts a 'const' shared pointer to 'mutable' shared pointer. (const_cast)
 *
 * @param  InSharedPtr  The shared pointer to cast
 */
template< class CastToType, class CastFromType, ESPMode Mode >
[[nodiscard]] UE_REWRITE TSharedPtr< CastToType, Mode > ConstCastSharedPtr( TSharedPtr< CastFromType, Mode > const& InSharedPtr )
{
	return TSharedPtr< CastToType, Mode >( InSharedPtr, SharedPointerInternals::FConstCastTag() );
}

template <class CastToType, class CastFromType, ESPMode Mode>
[[nodiscard]] UE_REWRITE TSharedPtr<CastToType, Mode> ConstCastSharedPtr(TSharedPtr<CastFromType, Mode>&& InSharedPtr)
{
	return TSharedPtr<CastToType, Mode>{MoveTemp(InSharedPtr), SharedPointerInternals::FConstCastTag{}};
}


/**
 * Casts a 'const' weak pointer to 'mutable' weak pointer. (const_cast)
 *
 * @param  InWeakPtr  The weak pointer to cast
 */
template< class CastToType, class CastFromType, ESPMode Mode >
[[nodiscard]] UE_REWRITE TWeakPtr< CastToType, Mode > ConstCastWeakPtr(TWeakPtr< CastFromType, Mode > const& InWeakPtr)
{
	return TWeakPtr< CastToType, Mode >(InWeakPtr, SharedPointerInternals::FConstCastTag());
}

template <class CastToType, class CastFromType, ESPMode Mode>
[[nodiscard]] UE_REWRITE TWeakPtr<CastToType, Mode> ConstCastWeakPtr(TWeakPtr<CastFromType, Mode>&& InWeakPtr)
{
	return TWeakPtr<CastToType, Mode>{MoveTemp(InWeakPtr), SharedPointerInternals::FConstCastTag{}};
}


/**
 * MakeShareable utility function.  Wrap object pointers with MakeShareable to allow them to be implicitly
 * converted to shared pointers!  This is useful in assignment operations, or when returning a shared
 * pointer from a function.
 */
// NOTE: The following is an Unreal extension to standard shared_ptr behavior
template< class ObjectType >
[[nodiscard]] inline SharedPointerInternals::TRawPtrProxy< ObjectType > MakeShareable( ObjectType* InObject )
{
	///////////////////////////////////////////////////////////////////////////////////////////////
	// If you get an 'ambiguous call' compile error in this function, it means you have multiple //
	// TSharedFromThis bases in your inheritance hierarchy.  This is not supported.              //
	///////////////////////////////////////////////////////////////////////////////////////////////

	if constexpr (IsDerivedFromSharedFromThis<ObjectType>())
	{
		// If this goes off, you should probably be using Ptr->AsShared() or Ptr->AsWeak() instead.
		checkf(!InObject || !InObject->DoesSharedInstanceExist(), TEXT("Trying to share an already-shared object"));
	}

	return SharedPointerInternals::TRawPtrProxy< ObjectType >( InObject );
}


/**
 * MakeShareable utility function.  Wrap object pointers with MakeShareable to allow them to be implicitly
 * converted to shared pointers!  This is useful in assignment operations, or when returning a shared
 * pointer from a function.
 */
// NOTE: The following is an Unreal extension to standard shared_ptr behavior
template< class ObjectType, class DeleterType >
[[nodiscard]] inline SharedPointerInternals::TRawPtrProxyWithDeleter< ObjectType, DeleterType > MakeShareable( ObjectType* InObject, DeleterType&& InDeleter )
{
	///////////////////////////////////////////////////////////////////////////////////////////////
	// If you get an 'ambiguous call' compile error in this function, it means you have multiple //
	// TSharedFromThis bases in your inheritance hierarchy.  This is not supported.              //
	///////////////////////////////////////////////////////////////////////////////////////////////

	if constexpr (IsDerivedFromSharedFromThis<ObjectType>())
	{
		// If this goes off, you should probably be using Ptr->AsShared() or Ptr->AsWeak() instead.
		checkf(!InObject || !InObject->DoesSharedInstanceExist(), TEXT("Trying to share an already-shared object"));
	}

	return SharedPointerInternals::TRawPtrProxyWithDeleter< ObjectType, DeleterType >( InObject, Forward< DeleterType >( InDeleter ) );
}

/**
 * MakeShared utility function.  Allocates a new ObjectType and reference controller in a single memory block.
 * Equivalent to std::make_shared.
 *
 * NOTE: If the constructor is private/protected you will need to utilize a private token.  Do not make SharedPointerInternals::TIntrusiveReferenceController a friend.
 *
 *     class FMyType
 *     {
 *     private:
 *         struct FPrivateToken { explicit FPrivateToken() = default; };
 *
 *     public:
 *         // This has an equivalent access level to a private constructor,
 *         // as only friends of FMyType will have access to FPrivateToken,
 *         // but MakeShared can legally call it since it's public.
 *         explicit FMyType(FPrivateToken, int32 Int, float Real, const TCHAR* String);
 *     };
 *
 *     // Won't compile if the caller doesn't have access to FMyType::FPrivateToken
 *     TSharedPtr<FMyType> Val = MakeShared<FMyType>(FMyType::FPrivateToken{}, 5, 3.14f, TEXT("Banana"));
 */
template <typename InObjectType, ESPMode InMode = ESPMode::ThreadSafe, typename... InArgTypes>
[[nodiscard]] inline TSharedRef<InObjectType, InMode> MakeShared(InArgTypes&&... Args)
{
	SharedPointerInternals::TIntrusiveReferenceController<InObjectType, InMode>* Controller = SharedPointerInternals::NewIntrusiveReferenceController<InMode, InObjectType>(Forward<InArgTypes>(Args)...);
	return UE::Core::Private::MakeSharedRef<InObjectType, InMode>(Controller->GetObjectPtr(), (SharedPointerInternals::TReferenceControllerBase<InMode>*)Controller);
}


/**
 * Given a TArray of TWeakPtr's, will remove any invalid pointers.
 * @param  PointerArray  The pointer array to prune invalid pointers out of
 */
template <class Type>
inline void CleanupPointerArray(TArray< TWeakPtr<Type> >& PointerArray)
{
	TArray< TWeakPtr<Type> > NewArray;
	for (int32 i = 0; i < PointerArray.Num(); ++i)
	{
		if (PointerArray[i].IsValid())
		{
			NewArray.Add(PointerArray[i]);
		}
	}
	PointerArray = NewArray;
}


/**
 * Given a TMap of TWeakPtr's, will remove any invalid pointers. Not the most efficient.
 * @param  PointerMap  The pointer map to prune invalid pointers out of
 */
template <class KeyType, class ValueType>
inline void CleanupPointerMap(TMap< TWeakPtr<KeyType>, ValueType >& PointerMap)
{
	TMap< TWeakPtr<KeyType>, ValueType > NewMap;
	for (typename TMap< TWeakPtr<KeyType>, ValueType >::TConstIterator Op(PointerMap); Op; ++Op)
	{
		const TWeakPtr<KeyType> WeakPointer = Op.Key();
		if (WeakPointer.IsValid())
		{
			NewMap.Add(WeakPointer, Op.Value());
		}
	}
	PointerMap = NewMap;
}

/**
* Computes a hash code for this object
*
* @param  InSharedRef  Shared pointer to compute hash code for
*
* @return  Hash code value
*/
template <typename ObjectType, ESPMode Mode>
[[nodiscard]] UE_REWRITE uint32 GetTypeHash( const TSharedRef< ObjectType, Mode >& InSharedRef )
{
	return ::PointerHash( &InSharedRef.Get() );
}

/**
* Computes a hash code for this object
*
* @param  InSharedPtr  Shared pointer to compute hash code for
*
* @return  Hash code value
*/
template <typename ObjectType, ESPMode Mode>
[[nodiscard]] UE_REWRITE uint32 GetTypeHash( const TSharedPtr< ObjectType, Mode >& InSharedPtr )
{
	return ::PointerHash( InSharedPtr.Get() );
}

/**
* Computes a hash code for this object
*
* @param  InWeakPtr  Weak pointer to compute hash code for
*
* @return  Hash code value
*/
template <typename ObjectType, ESPMode Mode>
[[nodiscard]] UE_REWRITE uint32 GetTypeHash( const TWeakPtr< ObjectType, Mode >& InWeakPtr )
{
	return InWeakPtr.GetWeakPtrTypeHash();
}

// Shared pointer testing
#include "Templates/SharedPointerTesting.inl" // IWYU pragma: export
