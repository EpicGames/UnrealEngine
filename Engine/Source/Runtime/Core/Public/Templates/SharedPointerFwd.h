// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h" // For "uint8"

/**
 * ESPMode is used select between either 'fast' or 'thread safe' shared pointer types.
 * This is only used by templates at compile time to generate one code path or another.
 */
enum class ESPMode : uint8
{
	/** Forced to be not thread-safe. */
	NotThreadSafe = 0,

	/** Thread-safe, never spin locks, but slower */
	ThreadSafe = 1
};

// IWYU pragma: begin_exports

// Forward declarations.  By default, thread safety features are turned on. (Mode = ESPMode::ThreadSafe).
// If you are more concerned with performance of ref-counting, you should use ESPMode::NotThreadSafe.
template< class ObjectType, ESPMode Mode = ESPMode::ThreadSafe > class TSharedRef;
template< class ObjectType, ESPMode Mode = ESPMode::ThreadSafe > class TSharedPtr;
template< class ObjectType, ESPMode Mode = ESPMode::ThreadSafe > class TWeakPtr;
template< class ObjectType, ESPMode Mode = ESPMode::ThreadSafe > class TSharedFromThis;

// IWYU pragma: end_exports
