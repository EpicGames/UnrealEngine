// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Included through SharedPointer.h

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "Templates/RemoveReference.h"
#include "Templates/SharedPointerFwd.h"
#include "Templates/TypeCompatibleBytes.h"
#include "AutoRTFM.h"
#include <atomic>
#include <type_traits>

/** Default behavior. */
#define THREAD_SANITISE_UNSAFEPTR 0

#if THREAD_SANITISE_UNSAFEPTR
	#define TSAN_SAFE_UNSAFEPTR
#else
	#define TSAN_SAFE_UNSAFEPTR TSAN_SAFE
#endif


/**
 * SharedPointerInternals contains internal workings of shared and weak pointers.  You should
 * hopefully never have to use anything inside this namespace directly.
 */
namespace SharedPointerInternals
{
	// Forward declarations
	template< ESPMode Mode > class FWeakReferencer;

	template <class SharedReferencerType, class ObjectType>
	void EnableSharedFromThis(SharedReferencerType& InSharedReferencer, ObjectType const* InObject);

	/** Dummy structures used internally as template arguments for typecasts */
	struct FStaticCastTag {};
	struct FConstCastTag {};

	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	struct FNullTag {};

	template <ESPMode Mode>
	class TReferenceControllerBase
	{
		using RefCountType = std::conditional_t<Mode == ESPMode::ThreadSafe, std::atomic<int32>, int32>;

	public:
		UE_FORCEINLINE_HINT explicit TReferenceControllerBase() = default;

		// Number of shared references to this object.  When this count reaches zero, the associated object
		// will be destroyed (even if there are still weak references!), but not the reference controller.
		//
		// This starts at 1 because we create reference controllers via the construction of a TSharedPtr,
		// and that is the first reference.  There is no point in starting at 0 and then incrementing it.
		RefCountType SharedReferenceCount{1};

		// Number of weak references to this object.  If there are any shared references, that counts as one
		// weak reference too.  When this count reaches zero, the reference controller will be deleted.
		//
		// This starts at 1 because it represents the shared reference that we are also initializing
		// SharedReferenceCount with.
		RefCountType WeakReferenceCount{1};

		/** Destroys the object associated with this reference counter.  */
		virtual void DestroyObject() = 0;

		virtual ~TReferenceControllerBase() = default;

		/** Returns the shared reference count */
		inline int32 GetSharedReferenceCount() const
		{
			if constexpr (Mode == ESPMode::ThreadSafe)
			{
				// A 'live' shared reference count is unstable by nature and so there's no benefit
				// to try and enforce memory ordering around the reading of it.
				//
				// This is equivalent to https://en.cppreference.com/w/cpp/memory/shared_ptr/use_count

				int32 Count = 0;

				UE_AUTORTFM_OPEN
					{
						// This reference count may be accessed by multiple threads
						Count = SharedReferenceCount.load(std::memory_order_relaxed);
					}
				;

				return Count;
			}
			else
			{
				return SharedReferenceCount;
			}
		}

		/** Checks if there is exactly one reference left to the object. */
		UE_FORCEINLINE_HINT bool IsUnique() const
		{
			return 1 == GetSharedReferenceCount();
		}

		/** Adds a shared reference to this counter */
		inline void AddSharedReference()
		{
			if constexpr (Mode == ESPMode::ThreadSafe)
			{
				// Incrementing a reference count with relaxed ordering is always safe because no other action is taken
				// in response to the increment, so there's nothing to order with.

#if defined(_MSC_VER) && (defined(_M_X64) || defined(_M_IX86))
				UE_AUTORTFM_OPEN
				{
					// We do a regular SC increment here because it maps to an _InterlockedIncrement (lock inc).
					// The codegen for a relaxed fetch_add is actually much worse under MSVC (lock xadd).
					++SharedReferenceCount;
				};
#else
				UE_AUTORTFM_OPEN
				{
					SharedReferenceCount.fetch_add(1, std::memory_order_relaxed);
				};
#endif

				// If the transaction would abort, we need to undo adding the shared reference.
				UE_AUTORTFM_ONABORT(this)
				{
					ReleaseSharedReference();
				};
			}
			else
			{
				++SharedReferenceCount;
			}
		}

		/**
		 * Adds a shared reference to this counter ONLY if there is already at least one reference
		 *
		 * @return  True if the shared reference was added successfully
		 */
		bool ConditionallyAddSharedReference()
		{
			if constexpr (Mode == ESPMode::ThreadSafe)
			{
				bool bSucceeded = false;

				UE_AUTORTFM_OPEN
				{
					// See AddSharedReference for the same reasons that std::memory_order_relaxed is used in this function.

					// Peek at the current shared reference count.  Remember, this value may be updated by
					// multiple threads.
					int32 OriginalCount = SharedReferenceCount.load(std::memory_order_relaxed);

					for (; ; )
					{
						if (OriginalCount == 0)
						{
							// Never add a shared reference if the pointer has already expired
							bSucceeded = false;
							break;
						}

						// Attempt to increment the reference count.
						//
						// We need to make sure that we never revive a counter that has already expired, so if the
						// actual value what we expected (because it was touched by another thread), then we'll try
						// again.  Note that only in very unusual cases will this actually have to loop.
						//
						// We do a weak read here because we require a loop and this is the recommendation:
						//
						// https://en.cppreference.com/w/cpp/atomic/atomic/compare_exchange
						//
						// > When a compare-and-exchange is in a loop, the weak version will yield better performance on some platforms.
						// > When a weak compare-and-exchange would require a loop and a strong one would not, the strong one is preferable
						if (SharedReferenceCount.compare_exchange_weak(OriginalCount, OriginalCount + 1, std::memory_order_relaxed))
						{
							bSucceeded = true;
							break;
						}
					}
				};

				// If we succeeded in taking a shared reference count, we need to undo that on an abort.
				if (bSucceeded)
				{
					UE_AUTORTFM_ONABORT(this)
					{
						ReleaseSharedReference();
					};
				}

				return bSucceeded;
			}
			else
			{
				if( SharedReferenceCount == 0 )
				{
					// Never add a shared reference if the pointer has already expired
					return false;
				}

				++SharedReferenceCount;
				return true;
			}
		}

		/** Releases a shared reference to this counter */
		inline void ReleaseSharedReference()
		{
			if constexpr (Mode == ESPMode::ThreadSafe)
			{
				UE_AUTORTFM_ONCOMMIT(this)
				{
					// std::memory_order_acq_rel is used here so that, if we do end up executing the destructor, it's not possible
					// for side effects from executing the destructor end up being visible before we've determined that the shared
					// reference count is actually zero.

					int32 OldSharedCount = SharedReferenceCount.fetch_sub(1, std::memory_order_acq_rel);
					checkSlow(OldSharedCount > 0);
					if (OldSharedCount == 1)
					{
						// Last shared reference was released!  Destroy the referenced object.
						DestroyObject();

						// No more shared referencers, so decrement the weak reference count by one.  When the weak
						// reference count reaches zero, this object will be deleted.
						ReleaseWeakReference();
					}
				};
			}
			else
			{
				checkSlow( SharedReferenceCount > 0 );

				if( --SharedReferenceCount == 0 )
				{
					// Last shared reference was released!  Destroy the referenced object.
					DestroyObject();

					// No more shared referencers, so decrement the weak reference count by one.  When the weak
					// reference count reaches zero, this object will be deleted.
					ReleaseWeakReference();
				}
			}
		}

		/** Release a shared reference, including nullptr check. Not inlined, for code size. */
		static FORCENOINLINE void ReleaseSharedReferenceNoInline(TReferenceControllerBase<Mode>* ReferenceController)
		{
			if (ReferenceController != nullptr)
			{
				// Tell the reference counter object that we're no longer referencing the object with
				// this shared pointer
				ReferenceController->ReleaseSharedReference();
			}
		}

		/** Adds a weak reference to this counter */
		inline void AddWeakReference()
		{
			if constexpr (Mode == ESPMode::ThreadSafe)
			{
				// See AddSharedReference for the same reasons that std::memory_order_relaxed is used in this function.

#if defined(_MSC_VER) && (defined(_M_X64) || defined(_M_IX86))
				UE_AUTORTFM_OPEN
					{
						// We do a regular SC increment here because it maps to an _InterlockedIncrement (lock inc).
						// The codegen for a relaxed fetch_add is actually much worse under MSVC (lock xadd).
						++WeakReferenceCount;
					};
#else
				UE_AUTORTFM_OPEN
					{
						WeakReferenceCount.fetch_add(1, std::memory_order_relaxed);
					};
#endif

				// If the transaction would abort, we need to undo adding the reference.
				UE_AUTORTFM_ONABORT(this)
					{
						ReleaseWeakReference();
					};
			}
			else
			{
				++WeakReferenceCount;
			}
		}

		/** Releases a weak reference to this counter */
		void ReleaseWeakReference()
		{
			if constexpr (Mode == ESPMode::ThreadSafe)
			{
				UE_AUTORTFM_ONCOMMIT(this)
					{
						// See ReleaseSharedReference for the same reasons that std::memory_order_acq_rel is used in this function.

						int32 OldWeakCount = WeakReferenceCount.fetch_sub(1, std::memory_order_acq_rel);
						checkSlow(OldWeakCount > 0);
						if (OldWeakCount == 1)
						{
							// Disable this if running clang's static analyzer. Passing shared pointers
							// and references to functions it cannot reason about, produces false
							// positives about use-after-free in the TSharedPtr/TSharedRef destructors.
#if !defined(__clang_analyzer__)
							// No more references to this reference count.  Destroy it!
							delete this;
#endif
						}
					};
			}
			else
			{
				checkSlow( WeakReferenceCount > 0 );

				if( --WeakReferenceCount == 0 )
				{
					// No more references to this reference count.  Destroy it!
#if !defined(__clang_analyzer__)
					delete this;
#endif
				}
			}
		}

		// Non-copyable
		TReferenceControllerBase(const TReferenceControllerBase&) = delete;
		TReferenceControllerBase& operator=(const TReferenceControllerBase&) = delete;
	};

	// A helper class that efficiently stores a custom deleter and is intended to be derived from. If the custom deleter is an empty class,
	// TDeleterHolder derives from it exploiting empty base optimisation (https://en.cppreference.com/w/cpp/language/ebo). Otherwise
	// it stores the custom deleter as a member to allow a function pointer to be used as a custom deleter (a function pointer can't 
	// be a base class)
	template <typename DeleterType, bool bIsZeroSize = std::is_empty_v<DeleterType>>
	struct TDeleterHolder : private DeleterType
	{
		explicit TDeleterHolder(DeleterType&& Arg)
			: DeleterType(MoveTemp(Arg))
		{
		}

		template <typename ObjectType>
		void InvokeDeleter(ObjectType * Object)
		{
			Invoke(*static_cast<DeleterType*>(this), Object);
		}
	};

	template <typename DeleterType>
	struct TDeleterHolder<DeleterType, false>
	{
		explicit TDeleterHolder(DeleterType&& Arg)
			: Deleter(MoveTemp(Arg))
		{
		}

		template <typename ObjectType>
		void InvokeDeleter(ObjectType * Object)
		{
			Invoke(Deleter, Object);
		}

	private:
		DeleterType Deleter;
	};

	template <typename ObjectType, typename DeleterType, ESPMode Mode>
	class TReferenceControllerWithDeleter : private TDeleterHolder<DeleterType>, public TReferenceControllerBase<Mode>
	{
	public:
		explicit TReferenceControllerWithDeleter(ObjectType* InObject, DeleterType&& Deleter)
			: TDeleterHolder<DeleterType>(MoveTemp(Deleter))
			, Object(InObject)
		{
		}

		virtual void DestroyObject() override
		{
			this->InvokeDeleter(Object);
		}

		// Non-copyable
		TReferenceControllerWithDeleter(const TReferenceControllerWithDeleter&) = delete;
		TReferenceControllerWithDeleter& operator=(const TReferenceControllerWithDeleter&) = delete;

	private:
		/** The object associated with this reference counter.  */
		ObjectType* Object;
	};

	template <typename ObjectType, ESPMode Mode>
	class TIntrusiveReferenceController : public TReferenceControllerBase<Mode>
	{
	public:
		template <typename... ArgTypes>
		explicit TIntrusiveReferenceController(ArgTypes&&... Args)
		{
			// If this fails to compile when trying to call MakeShared with a non-public constructor,
			// do not make SharedPointerInternals::TIntrusiveReferenceController a friend.
			//
			// Instead, prefer this pattern:
			//
			//     class FMyType
			//     {
			//     private:
			//         struct FPrivateToken { explicit FPrivateToken() = default; };
			//
			//     public:
			//         // This has an equivalent access level to a private constructor,
			//         // as only friends of FMyType will have access to FPrivateToken,
			//         // but MakeShared can legally call it since it's public.
			//         explicit FMyType(FPrivateToken, int32 Int, float Real, const TCHAR* String);
			//     };
			//
			//     // Won't compile if the caller doesn't have access to FMyType::FPrivateToken
			//     TSharedPtr<FMyType> Val = MakeShared<FMyType>(FMyType::FPrivateToken{}, 5, 3.14f, TEXT("Banana"));
			//
			::new ((void*)&ObjectStorage) ObjectType(Forward<ArgTypes>(Args)...);
		}

		ObjectType* GetObjectPtr() const
		{
			return (ObjectType*)&ObjectStorage;
		}

		virtual void DestroyObject() override
		{
			DestructItem((ObjectType*)&ObjectStorage);
		}

		// Non-copyable
		TIntrusiveReferenceController(const TIntrusiveReferenceController&) = delete;
		TIntrusiveReferenceController& operator=(const TIntrusiveReferenceController&) = delete;

	private:
		/** The object associated with this reference counter.  */
		mutable TTypeCompatibleBytes<ObjectType> ObjectStorage;
	};


	/** Deletes an object via the standard delete operator */
	template <typename Type>
	struct DefaultDeleter
	{
		UE_FORCEINLINE_HINT void operator()(Type* Object) const
		{
			delete Object;
		}
	};

	/** Creates a reference controller which just calls delete */
	template <ESPMode Mode, typename ObjectType>
	inline TReferenceControllerBase<Mode>* NewDefaultReferenceController(ObjectType* Object)
	{
		return new TReferenceControllerWithDeleter<ObjectType, DefaultDeleter<ObjectType>, Mode>(Object, DefaultDeleter<ObjectType>());
	}

	/** Creates a custom reference controller with a specified deleter */
	template <ESPMode Mode, typename ObjectType, typename DeleterType>
	inline TReferenceControllerBase<Mode>* NewCustomReferenceController(ObjectType* Object, DeleterType&& Deleter)
	{
		return new TReferenceControllerWithDeleter<ObjectType, typename TRemoveReference<DeleterType>::Type, Mode>(Object, Forward<DeleterType>(Deleter));
	}

	/** Creates an intrusive reference controller */
	template <ESPMode Mode, typename ObjectType, typename... ArgTypes>
	inline TIntrusiveReferenceController<ObjectType, Mode>* NewIntrusiveReferenceController(ArgTypes&&... Args)
	{
		return new TIntrusiveReferenceController<ObjectType, Mode>(Forward<ArgTypes>(Args)...);
	}


	/** Proxy structure for implicitly converting raw pointers to shared/weak pointers */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <class ObjectType>
	struct TRawPtrProxy
	{
		/** The object pointer */
		ObjectType* Object;

		/** Construct implicitly from a nullptr */
		UE_FORCEINLINE_HINT TRawPtrProxy( TYPE_OF_NULLPTR )
			: Object( nullptr )
		{
		}

		/** Construct explicitly from an object */
		explicit UE_FORCEINLINE_HINT TRawPtrProxy( ObjectType* InObject )
			: Object( InObject )
		{
		}
	};


	/** Proxy structure for implicitly converting raw pointers to shared/weak pointers */
	// NOTE: The following is an Unreal extension to standard shared_ptr behavior
	template <class ObjectType, typename DeleterType>
	struct TRawPtrProxyWithDeleter
	{
		/** The object pointer */
		ObjectType* Object;

		/** The deleter object */
		DeleterType Deleter;

		/** Construct implicitly from an object and a custom deleter */
		inline TRawPtrProxyWithDeleter( ObjectType* InObject, const DeleterType& InDeleter )
			: Object ( InObject )
			, Deleter( InObject, InDeleter )
		{
		}

		/** Construct implicitly from an object and a custom deleter */
		inline TRawPtrProxyWithDeleter( ObjectType* InObject, DeleterType&& InDeleter )
			: Object ( InObject )
			, Deleter( MoveTemp( InDeleter ) )
		{
		}
	};


	/**
	 * FSharedReferencer is a wrapper around a pointer to a reference controller that is used by either a
	 * TSharedRef or a TSharedPtr to keep track of a referenced object's lifetime
	 */
	template< ESPMode Mode >
	class FSharedReferencer
	{
	public:

		/** Constructor for an empty shared referencer object */
		UE_FORCEINLINE_HINT FSharedReferencer()
			: ReferenceController( nullptr )
		{ }

		/** Constructor that references the specified controller, without incrementing the shared count */
		inline explicit FSharedReferencer( TReferenceControllerBase<Mode>* InReferenceController )
			: ReferenceController( InReferenceController )
		{ }
		
		/** Copy constructor creates a new reference to the existing object */
		inline FSharedReferencer( FSharedReferencer const& InSharedReference )
			: ReferenceController( InSharedReference.ReferenceController )
		{
			// If the incoming reference had an object associated with it, then go ahead and increment the
			// shared reference count
			if( ReferenceController != nullptr )
			{
				ReferenceController->AddSharedReference();
			}
		}

		/** Move constructor creates no new references */
		inline FSharedReferencer( FSharedReferencer&& InSharedReference )
			: ReferenceController( InSharedReference.ReferenceController )
		{
			InSharedReference.ReferenceController = nullptr;
		}

		/** Creates a shared referencer object from a weak referencer object.  This will only result
			in a valid object reference if the object already has at least one other shared referencer. */
		FSharedReferencer(FWeakReferencer< Mode > const& InWeakReference)
			: ReferenceController(InWeakReference.ReferenceController)
		{
			// If the incoming reference had an object associated with it, then go ahead and increment the
			// shared reference count
			if (ReferenceController != nullptr)
			{
				// Attempt to elevate a weak reference to a shared one.  For this to work, the object this
				// weak counter is associated with must already have at least one shared reference.  We'll
				// never revive a pointer that has already expired!
				if (!ReferenceController->ConditionallyAddSharedReference())
				{
					ReferenceController = nullptr;
				}
			}
		}

		/** Creates a shared referencer object from a weak referencer object.  This will only result
		    in a valid object reference if the object already has at least one other shared referencer. */
		FSharedReferencer( FWeakReferencer< Mode >&& InWeakReference )
			: ReferenceController( InWeakReference.ReferenceController )
		{
			// If the incoming reference had an object associated with it, then go ahead and increment the
			// shared reference count
			if( ReferenceController != nullptr )
			{
				// Attempt to elevate a weak reference to a shared one.  For this to work, the object this
				// weak counter is associated with must already have at least one shared reference.  We'll
				// never revive a pointer that has already expired!
				if( !ReferenceController->ConditionallyAddSharedReference() )
				{
					ReferenceController = nullptr;
				}

				// Tell the reference counter object that we're no longer referencing the object with
				// this weak pointer
				InWeakReference.ReferenceController->ReleaseWeakReference();
				InWeakReference.ReferenceController = nullptr;
			}
		}

		/** Destructor. */
		inline ~FSharedReferencer()
		{
			TReferenceControllerBase<Mode>::ReleaseSharedReferenceNoInline(ReferenceController);
		}

		/** Assignment operator adds a reference to the assigned object.  If this counter was previously
		    referencing an object, that reference will be released. */
		inline FSharedReferencer& operator=( FSharedReferencer const& InSharedReference )
		{
			// Make sure we're not be reassigned to ourself!
			auto NewReferenceController = InSharedReference.ReferenceController;
			if( NewReferenceController != ReferenceController )
			{
				// First, add a shared reference to the new object
				if( NewReferenceController != nullptr )
				{
					NewReferenceController->AddSharedReference();
				}

				// Release shared reference to the old object
				if( ReferenceController != nullptr )
				{
					ReferenceController->ReleaseSharedReference();
				}

				// Assume ownership of the assigned reference counter
				ReferenceController = NewReferenceController;
			}

			return *this;
		}

		/** Move assignment operator adds no references to the assigned object.  If this counter was previously
		    referencing an object, that reference will be released. */
		inline FSharedReferencer& operator=( FSharedReferencer&& InSharedReference )
		{
			// Make sure we're not be reassigned to ourself!
			auto NewReferenceController = InSharedReference.ReferenceController;
			auto OldReferenceController = ReferenceController;
			if( NewReferenceController != OldReferenceController )
			{
				// Assume ownership of the assigned reference counter
				InSharedReference.ReferenceController = nullptr;
				ReferenceController                   = NewReferenceController;

				// Release shared reference to the old object
				if( OldReferenceController != nullptr )
				{
					OldReferenceController->ReleaseSharedReference();
				}
			}

			return *this;
		}

		/**
		 * Tests to see whether or not this shared counter contains a valid reference
		 *
		 * @return  True if reference is valid
		 */
		UE_FORCEINLINE_HINT const bool IsValid() const
		{
			return ReferenceController != nullptr;
		}

		/**
		 * Returns the number of shared references to this object (including this reference.)
		 *
		 * @return  Number of shared references to the object (including this reference.)
		 */
		UE_FORCEINLINE_HINT const int32 GetSharedReferenceCount() const
		{
			return ReferenceController != nullptr ? ReferenceController->GetSharedReferenceCount() : 0;
		}

		/**
		 * Returns true if this is the only shared reference to this object.  Note that there may be
		 * outstanding weak references left.
		 *
		 * @return  True if there is only one shared reference to the object, and this is it!
		 */
		UE_FORCEINLINE_HINT const bool IsUnique() const
		{
			return ReferenceController != nullptr && ReferenceController->IsUnique();
		}

	private:

 		// Expose access to ReferenceController to FWeakReferencer
		template< ESPMode OtherMode > friend class FWeakReferencer;

		template <class SharedReferencerType, class OtherType>
		friend void EnableSharedFromThis(SharedReferencerType& InSharedReferencer, OtherType const* InObject);

	private:

		/** Pointer to the reference controller for the object a shared reference/pointer is referencing */
		TReferenceControllerBase<Mode>* ReferenceController;
	};


	/**
	 * FWeakReferencer is a wrapper around a pointer to a reference controller that is used
	 * by a TWeakPtr to keep track of a referenced object's lifetime.
	 */
	template< ESPMode Mode >
	class FWeakReferencer
	{
	public:

		/** Default constructor with empty counter */
		UE_FORCEINLINE_HINT FWeakReferencer()
			: ReferenceController( nullptr )
		{ }

		/** Constructor that references the specified controller, without incrementing the weak count */
		explicit FWeakReferencer(TReferenceControllerBase<Mode>* InReferenceController)
			: ReferenceController{InReferenceController}
		{
		}

		/** Construct a weak referencer object from another weak referencer */
		inline FWeakReferencer( FWeakReferencer const& InWeakRefCountPointer )
			: ReferenceController( InWeakRefCountPointer.ReferenceController )
		{
			// If the weak referencer has a valid controller, then go ahead and add a weak reference to it!
			if( ReferenceController != nullptr )
			{
				ReferenceController->AddWeakReference();
			}
		}

		/** Construct a weak referencer object from an rvalue weak referencer */
		inline FWeakReferencer( FWeakReferencer&& InWeakRefCountPointer )
			: ReferenceController( InWeakRefCountPointer.ReferenceController )
		{
			InWeakRefCountPointer.ReferenceController = nullptr;
		}

		/** Construct a weak referencer object from a shared referencer object */
		inline FWeakReferencer( FSharedReferencer< Mode > const& InSharedRefCountPointer )
			: ReferenceController( InSharedRefCountPointer.ReferenceController )
		{
			// If the shared referencer had a valid controller, then go ahead and add a weak reference to it!
			if( ReferenceController != nullptr )
			{
				ReferenceController->AddWeakReference();
			}
		}

		/** Destructor. */
		inline ~FWeakReferencer()
		{
			if( ReferenceController != nullptr )
			{
				// Tell the reference counter object that we're no longer referencing the object with
				// this weak pointer
				ReferenceController->ReleaseWeakReference();
			}
		}
		
		/** Assignment operator from a weak referencer object.  If this counter was previously referencing an
		    object, that reference will be released. */
		inline FWeakReferencer& operator=( FWeakReferencer const& InWeakReference )
		{
			AssignReferenceController( InWeakReference.ReferenceController );

			return *this;
		}

		/** Assignment operator from an rvalue weak referencer object.  If this counter was previously referencing an
		    object, that reference will be released. */
		inline FWeakReferencer& operator=( FWeakReferencer&& InWeakReference )
		{
			auto OldReferenceController         = ReferenceController;
			ReferenceController                 = InWeakReference.ReferenceController;
			InWeakReference.ReferenceController = nullptr;
			if( OldReferenceController != nullptr )
			{
				OldReferenceController->ReleaseWeakReference();
			}

			return *this;
		}

		/** Assignment operator from a shared reference counter.  If this counter was previously referencing an
		   object, that reference will be released. */
		inline FWeakReferencer& operator=( FSharedReferencer< Mode > const& InSharedReference )
		{
			AssignReferenceController( InSharedReference.ReferenceController );

			return *this;
		}

		/**
		 * Tests to see whether or not this weak counter contains a valid reference
		 *
		 * @return  True if reference is valid
		 */
		UE_FORCEINLINE_HINT const bool IsValid() const
		{
			return ReferenceController != nullptr && ReferenceController->GetSharedReferenceCount() > 0;
		}

	private:

		/** Assigns a new reference controller to this counter object, first adding a reference to it, then
		    releasing the previous object. */
		inline void AssignReferenceController( TReferenceControllerBase<Mode>* NewReferenceController )
		{
			// Only proceed if the new reference counter is different than our current
			if( NewReferenceController != ReferenceController )
			{
				// First, add a weak reference to the new object
				if( NewReferenceController != nullptr )
				{
					NewReferenceController->AddWeakReference();
				}

				// Release weak reference to the old object
				if( ReferenceController != nullptr )
				{
					ReferenceController->ReleaseWeakReference();
				}

				// Assume ownership of the assigned reference counter
				ReferenceController = NewReferenceController;
			}
		}

	private:

 		/** Expose access to ReferenceController to FSharedReferencer. */
		template< ESPMode OtherMode > friend class FSharedReferencer;

	private:

		/** Pointer to the reference controller for the object a TWeakPtr is referencing */
		TReferenceControllerBase<Mode>* ReferenceController;
	};
}


namespace UE::Core::Private
{
	// This base class only exists to implement IsDerivedFromSharedFromThis
	struct FSharedFromThisBase
	{
	};
}

template <typename T>
constexpr bool IsDerivedFromSharedFromThis()
{
	return std::is_base_of_v<UE::Core::Private::FSharedFromThisBase, T>;
}


namespace SharedPointerInternals
{
	/** Templated helper function (const) that creates a shared reference or pointer from an object instance */
	template <class SharedReferencerType, class ObjectType>
	void EnableSharedFromThis(SharedReferencerType& InSharedReferencer, ObjectType const* InObject)
	{
		///////////////////////////////////////////////////////////////////////////////////////////////
		// If you get an 'ambiguous call' compile error in this function, it means you have multiple //
		// TSharedFromThis bases in your inheritance hierarchy.  This is not supported.              //
		///////////////////////////////////////////////////////////////////////////////////////////////

		if constexpr (IsDerivedFromSharedFromThis<ObjectType>())
		{
			if (InObject)
			{
				InObject->UpdateWeakReferenceInternal(InSharedReferencer.ReferenceController);
			}
		}
	}
}
