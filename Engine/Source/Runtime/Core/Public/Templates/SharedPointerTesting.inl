// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/** Toggle this define to enable shared pointer testing features */
#define WITH_SHARED_POINTER_TESTS 0 && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

#if WITH_SHARED_POINTER_TESTS

enum class ESPMode : uint8;
template<typename KeyType,typename ValueType,typename SetAllocator ,typename KeyFuncs > class TMap;

/**
 * Shared pointer testing suite
 */
namespace SharedPointerTesting
{
	/**
	 * Executes various shared pointer tests.  Note that some tests may require the programmer to enable
	 * some #ifdef statements to verify that code that is not expected to compile in fact does not.
	 */
	template< ESPMode Mode >		// Used to test *with* and *without* thread-safety features
	static void TestSharedPointer()
	{
		{
			// Empty shared ptr, not valid to deference
			TSharedPtr< bool, Mode > MyEmptyBoolPtr;

			// Test validity check
			check( !MyEmptyBoolPtr.IsValid() );

			// Test access to raw pointer
			if( MyEmptyBoolPtr.Get() == nullptr )
			{
				// ...
			}
		}

		{
			// Construct owned shared pointer from object instance
			TSharedPtr< int32, Mode > MyIntSharedPtr( new int32( 123 ) );

			// Test validity check
			check( MyIntSharedPtr.IsValid() );

			// Check uniqueness
			check( MyIntSharedPtr.IsUnique() );

			// Test dereference operator
			const int32 DeferenceTest = *MyIntSharedPtr;

			// Test reference count zero destructing owned object
			MyIntSharedPtr.Reset();


			// Check number of references
			check( MyIntSharedPtr.GetSharedReferenceCount() == 0 );

			// MyIntSharedPtr goes out of scope, but object already deleted, nothing to do
		}

		{
			// Test implicit conversion (not currently allowed because constructor is explicit!)
#if 0
			TSharedPtr< double, Mode > MyBool = 45.0;
#endif
		}

		{
			// Test copying shared reference
			TSharedPtr< bool, Mode > FirstBoolRef( new bool( false ) );
			TSharedPtr< bool, Mode > SecondBoolRef( FirstBoolRef );
		}

		{
			// Test copying shared reference using = operator
			TSharedPtr< bool, Mode > FirstBoolRef( new bool( false ) );
			TSharedPtr< bool, Mode > SecondBoolRef = FirstBoolRef;
		}

		{
			// Test arrow operator
			struct FSharedTest
			{
				bool bFoo;
			};
			TSharedPtr< FSharedTest, Mode > SharedArray( new FSharedTest() );
			SharedArray->bFoo = true;

			// Test dereference operator
			( *SharedArray ).bFoo = false;

			// Create an additional reference to an existing shared pointer
			TSharedPtr< FSharedTest, Mode > OtherSharedArrayReference( SharedArray );

			// Release original reference (object should not be destroyed)
			SharedArray.Reset();

			// NOTE: OtherSharedArrayReference goes out of scope here (object is destroyed)
		}

		{
			// Test casting
			class FBase
			{
				bool bFoo;
			};
			class FDerived
				: public FBase
			{ };

			{
				// Explicit downcast to derived shared ptr
				TSharedPtr< FBase, Mode > DerivedAsBasePtr( new FDerived() );
				TSharedPtr< FDerived, Mode > DerivedPtr( StaticCastSharedPtr< FDerived >( DerivedAsBasePtr ) );
			}

			{
				// Initialize base from derived (implicit upcast)
				TSharedPtr< FDerived, Mode > DerivedPtr( new FDerived() );
				TSharedPtr< FBase, Mode > BasePtr( DerivedPtr );
			}

			{
				// Assign derived to base (implicit upcast)
				TSharedPtr< FDerived, Mode > DerivedPtr( new FDerived() );
				TSharedPtr< FBase, Mode > BasePtr = DerivedPtr;
			}
		}

		// Create a shared pointer to nullptr.  Consistent with std::shared_ptr, this results
		// in a non-empty TSharedPtr (heap allocated reference count)
		{
			bool* Foo = nullptr;
			TSharedPtr< bool, Mode > NullPtr( Foo );
			check( !NullPtr.IsValid() );
		}

		// Simple validity test syntax
		{
			TSharedPtr< bool, Mode > BoolPtr( new bool( true ) );
			check( BoolPtr.IsValid() );
		}

		// Empty weak pointer
		{
			TWeakPtr< bool, Mode > EmptyBoolWeakPtr;

			// Pin should fail on empty weak ptr
			check( !EmptyBoolWeakPtr.Pin().IsValid() );
		}

		// Create weak pointer from shared pointer
		{
			TSharedPtr< int32, Mode > SharedInt( new int32( 64 ) );
			TWeakPtr< int32, Mode > WeakInt( SharedInt );

			// Pin should succeed on this valid weak ptr
			check( WeakInt.Pin().IsValid() );
		}

		// Create weak pointer from shared pointer (assignment)
		{
			TSharedPtr< int32, Mode > SharedInt( new int32( 64 ) );
			TWeakPtr< int32, Mode > WeakInt = SharedInt;

			// Pin should succeed on this valid weak ptr
			check( WeakInt.Pin().IsValid() );

			// Reset a weak pointer
			WeakInt.Reset();
			check( !WeakInt.Pin().IsValid() );
		}

		// Test weak pointer becoming invalid
		{
			TSharedPtr< int32, Mode > SharedInt( new int32( 64 ) );
			TWeakPtr< int32, Mode > WeakInt = SharedInt;
			SharedInt.Reset();
			check( !WeakInt.Pin().IsValid() );
		}

		// Compare shared pointers
		{
			TSharedPtr< int32, Mode > SharedA( new int32( 64 ) );
			TSharedPtr< int32, Mode > SharedB( new int32( 21 ) );
			TSharedPtr< int32, Mode > SharedC( SharedB );

			check( !( SharedA == SharedB ) );
			check( SharedA != SharedB );
			check( SharedB == SharedC );
		}

		// Compare weak pointers
		{
			TSharedPtr< int32, Mode > SharedA( new int32( 64 ) );
			TSharedPtr< int32, Mode > SharedB( new int32( 21 ) );

			TWeakPtr< int32, Mode > WeakA( SharedA );
			TWeakPtr< int32, Mode > WeakB( SharedB  );
			TWeakPtr< int32, Mode > WeakC( SharedB  );

			{
				check( !( WeakA.Pin() == WeakB.Pin() ) );
				check( WeakA.Pin() != WeakB.Pin() );
				check( WeakB.Pin() == WeakC.Pin() );
			}

			// NOTE: Weak pointer direct comparisons not supported (consistent with std::weak_ptr)
#if 0		// Should not compile
			{
				check( !( WeakA == WeakB ) );
				check( WeakA != WeakB );
				check( WeakB == WeakC );
			}
#endif
		}

		// Test 'const'
		{
			TSharedPtr< const int32, Mode > IntPtr( new int32( 10 ) );
			TSharedPtr< const float, Mode > FloatPtrA( new float( 1.0f ) );
			TSharedPtr< const float, Mode > FloatPtrB( new float( 2.0f ) );
			
			if( FloatPtrA == FloatPtrB )
			{
			}

#if 0		// Won't compile as int32 is not compatible with float
			if( FloatPtrB == IntPtr )
			{
			}
#endif

			// Assigning const pointers (references only, this is OK!)
			FloatPtrA = FloatPtrB;

			// Test const conversion (not allowed!)
			TSharedPtr< float, Mode > MutableFloat( new float( 123.0f ) );
#if 0		// Won't compile as implicit const_cast is not allowed
			MutableFloat = FloatPtrA;
#endif

			// Test conversion from mutable to const (this is OK!)
			FloatPtrA = MutableFloat;

			if( FloatPtrB.IsValid() )
			{
#if 0			// Won't compile as value is const
				*FloatPtrB = 10.0f;
#endif
			}

			TWeakPtr< const float, Mode > ConstWeakFloat = FloatPtrA;	// Preserving const
#if 0		// Won't compile as value is const
			*ConstWeakFloat.Pin() = 20.0f;
#endif

			// Test implicit const_cast (not allowed!)
			TWeakPtr< float, Mode > WeakFloat;
#if 0		// Won't compile as const cannot convert to mutable implicitly
			WeakFloat = FloatPtrB;	// NOTE: 'const' not preserved here
#endif
			
			// Test const cast
			WeakFloat = ConstCastSharedPtr< float >( FloatPtrB );

			*WeakFloat.Pin() = 20.0f;
		}

		// Test forward declaring a smart pointer to an incomplete type
		{
			TSharedPtr< struct FBarFoo, Mode > VecPtr;
			struct FBarFoo
			{
				int32 Val;
			};
			VecPtr = TSharedPtr< FBarFoo, Mode >( new FBarFoo() );
			VecPtr->Val = 20;
		}
		
		// Test Unreal non-standard extensions (expanded syntax)
		{
			// Initialize with nullptr
			TSharedPtr< bool, Mode > EmptyPtr( nullptr );
			TSharedPtr< float, Mode > FloatPtr = nullptr;

			// Initialize with nullptr
			TWeakPtr< bool, Mode > EmptyWeakPtr( nullptr );
			TWeakPtr< float, Mode > FloatWeakPtr = nullptr;

			// Reassign to nullptr directly (instead of calling Reset)
			FloatPtr = TSharedPtr< float, Mode >( new float( 0.1f ) );
			FloatPtr = nullptr;

			// Test implicit construction helper (MakeShareable)
			FloatPtr = MakeShareable( new float( 30.0f ) );
			TSharedPtr< double, Mode >( MakeShareable( new double( 2.0 ) ) );

			struct FFooBar
			{
				// Test function return value
				TSharedPtr< float, Mode > GetFloatMember()
				{
					return FloatVal;
				}

				TSharedPtr< float, Mode > FloatVal;


				// Test implicit construction on return
				TSharedPtr< float, Mode > GetFloatValue()
				{
					return MakeShareable( new float( 123.0f ) );
				}
			};
		}

		// Test TSharedRef
		{
			// Empty TSharedRef is not allowed
			{
#if 0			// Won't compile as TSharedRef has no default constructor
				TSharedRef< bool, Mode > EmptyRef;
#endif

#if 0			// Won't compile as TSharedRef has no implicit constructor that a pointer (including nullptr)
				TSharedRef< bool, Mode > NullRef = nullptr;
#endif

				// TSharedRef initialized in constructor
				{
					TSharedRef< float, Mode > FloatRef( new float( 123.0f ) );
				}

				// Test reference access
				{
					TSharedRef< float, Mode > FloatRef( new float( 123.0f ) );

					// Grab a C++ reference to the float object
					const float& MyFloat = *FloatRef;

					// Grab another C++ reference to the float object
					const float& MyFloat2 = FloatRef.Get();
				}

				// Implicit conversion to TSharedRef is never allowed (use MakeShareable!)
				{
#if 0				// Won't compile as constructor is explicit
					TSharedRef< float, Mode > FloatRef = new float( 123.0f );
#endif
				}

				// Test MakeShareable with TSharedRef
				{
					TSharedRef< float, Mode > FloatRef = MakeShareable( new float( 123.0f ) );
				}

				// New shared ref with nullptr object pointer is not allowed!  The following code will compile
				// correctly, but will trigger a runtime assertion.
				if( 0 )
				{
					// NOTE: The following code is unsafe and will trigger an assert
					int32* NullInt = nullptr;
					TSharedRef< int32, Mode > RefWithNullObject( NullInt );
				}

				// Implicit conversion from a TSharedRef to a TSharedPtr (always OK)
				{
					TSharedRef< int32, Mode > MySharedRef( new int32( 1 ) );
					TSharedPtr< int32, Mode > MySharedPtr( MySharedRef );
				}

				// Explicit TSharedRef construction from a TSharedPtr is not allowed
				{
					TSharedPtr< int32, Mode > MySharedPtr( new int32( 1 ) );
#if 0				// Won't compile as this constructor is intentionally private.  Use ToSharedRef() instead!
					TSharedRef< int32, Mode > MySharedRef( MySharedPtr );
#endif
				}

				// Explicit TSharedRef assignment from a TSharedPtr is not allowed
				{
					TSharedPtr< int32, Mode > MySharedPtr( new int32( 1 ) );
#if 0				// Won't compile as this constructor is intentionally private.  Use ToSharedRef() instead!
					TSharedRef< int32, Mode > MySharedRef = MySharedPtr;
#endif
				}

				// Conversion from a TSharedPtr to a TSharedRef.  Only safe when ( MySharedPtr.IsValid() == true )
				{
					TSharedPtr< int32, Mode > MySharedPtr( new int32( 1 ) );
					TSharedRef< int32, Mode > MySharedRef( MySharedPtr.ToSharedRef() );
				}

				// Conversion from an invalid TSharedPtr to a TSharedRef.  Never safe, will assert at runtime.
				if( 0 )		// Will trigger a runtime assert as MySharedPtr.IsValid() == false
				{
					int32* NullInt = nullptr;
					TSharedPtr< int32, Mode > MySharedPtr( NullInt );
					TSharedRef< int32, Mode > MySharedRef( MySharedPtr.ToSharedRef() );
				}

				// TSharedRef reassignment.  Safe as long as new object is not nullptr.
				{
					TSharedRef< int32, Mode > IntRef( new int32( 10 ) );
					IntRef = TSharedRef< int32, Mode >( new int32( 20 ) );
				}

				// TSharedRef reassignment to nullptr object is not allowed (asserts at runtime)
				if( 0 )
				{
					TSharedRef< int32, Mode > IntRef( new int32( 10 ) );
					int32* NullInt = nullptr;
					IntRef = TSharedRef< int32, Mode >( NullInt );
				}

				// Get a weak pointer from a shared ref.  Always OK!
				{
					TSharedRef< int32, Mode > IntRef( new int32( 99 ) );
					TWeakPtr< int32, Mode > WeakInt = IntRef;
					if( WeakInt.IsValid() )
					{
						// ...
					}
				}

				// Compare two shared refs
				{
					TSharedRef< int32, Mode > IntRef1( new int32( 99 ) );
					TSharedRef< int32, Mode > IntRef2( new int32( 21 ) );
					if( IntRef1 == IntRef2 )
					{
						// ...
					}
					if( IntRef1 != IntRef2 )
					{
						// ...
					}
				}

				// Compare a shared pointer with a shared ref
				{
					// @todo: Use complex types, not int32 (has operator==)!!
					TSharedRef< int32, Mode > IntRef( new int32( 21 ) );
					TSharedPtr< int32, Mode > IntPtr( IntRef );
					
					// Pointer is equal to reference because they point to the same valid object
					check( IntRef == IntPtr && IntPtr == IntRef );
					{
						// ...
					}

					// Test not equal operator
					check( !( IntRef != IntPtr || IntPtr != IntRef ) );
					{
						// ...
					}

					// Null pointer is never equal to a reference
					TSharedPtr< int32, Mode > NullPtr;
					check( !( IntRef == NullPtr ) && ( IntRef != NullPtr ) );
					{
						// ...
					}
				}
			}
		}

		// SharedFromThis
		{
			class FMyClass
				: public TSharedFromThis< FMyClass, Mode >
			{
			public:
				TSharedRef< FMyClass, Mode > GetSelfAsShared()
				{
					return AsShared();
				}
			};

			// Grab shared pointer to stack-allocated class, before ever assigning it to
			// a shared pointer reference.  This will trigger an assertion!
			if( 0 )
			{
				FMyClass MyClass;
				TSharedRef< FMyClass, Mode > TheClassPtr( MyClass.GetSelfAsShared() );
			}

			TSharedPtr< FMyClass, Mode > TheClassPtr1( new FMyClass() );
			{
				FMyClass* MyClass = TheClassPtr1.Get();
				TSharedRef< FMyClass, Mode > TheClassPtr2( MyClass->GetSelfAsShared() );
			}
		}

		// Hash tables
		{
			TSharedRef< int32, Mode > FooRef( new int32( 1 ) );
			TSharedPtr< int32, Mode > FooPtr( FooRef );

			// Map shared pointers to a type
			TMap< TSharedPtr< int32, Mode >, bool > SharedPointerHash;
			SharedPointerHash.Add( FooPtr, true );

			// Map shared refs to a type
			TMap< TSharedRef< int32, Mode >, bool > SharedRefHash;
			SharedRefHash.Add( FooRef, true );
			const bool Value = SharedRefHash.FindRef( FooRef );

			// Map a type to shared refs
			{
				TMap< int32, TSharedRef< int32, Mode > > SharedRefValueHash;
				SharedRefValueHash.Add(10, FooRef);
				const int32* FoundKey = SharedRefValueHash.FindKey(FooRef);
				check(FoundKey != nullptr && *FoundKey == 10);
			}

			// Maps and 'const'
			{
				TSharedRef< const int32, Mode > ConstFooRef( new int32( 1 ) );
				TMap< int32, TSharedRef< const int32, Mode > > ConstSharedRefValueHash;
				ConstSharedRefValueHash.Add( 10, ConstFooRef );
				const int32* FoundKey = ConstSharedRefValueHash.FindKey( ConstFooRef );
				check( FoundKey != nullptr && *FoundKey == 10 );
			}
		}
	}
}


#endif		// WITH_SHARED_POINTER_TESTS
