// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Algo/BinarySearch.h"
#include "Algo/Sort.h"
#include "HAL/PlatformMath.h"
#include "Templates/Less.h"

/**
 * Helper class for dereferencing pointer types in Sort function
 */
template<typename T, class PREDICATE_CLASS> 
struct TDereferenceWrapper
{
	const PREDICATE_CLASS& Predicate;

	TDereferenceWrapper( const PREDICATE_CLASS& InPredicate )
		: Predicate( InPredicate ) {}
  
	/** Pass through for non-pointer types */
	UE_FORCEINLINE_HINT bool operator()( T& A, T& B ) { return Predicate( A, B ); } 
	UE_FORCEINLINE_HINT bool operator()( const T& A, const T& B ) const { return Predicate( A, B ); } 
};
/** Partially specialized version of the above class */
template<typename T, class PREDICATE_CLASS> 
struct TDereferenceWrapper<T*, PREDICATE_CLASS>
{
	const PREDICATE_CLASS& Predicate;

	TDereferenceWrapper( const PREDICATE_CLASS& InPredicate )
		: Predicate( InPredicate ) {}
  
	/** Dereference pointers */
	UE_FORCEINLINE_HINT bool operator()( T* A, T* B ) const 
	{
		return Predicate( *A, *B ); 
	} 
};

/**
 * Wraps a range into a container like interface to satisfy the GetData and GetNum global functions.
 * We're not using TArrayView since it calls ::Sort creating a circular dependency.
 */
template <typename T>
struct TArrayRange
{
	TArrayRange(T* InPtr, int32 InSize)
		: Begin(InPtr)
		, Size(InSize)
	{
	}

	T* GetData() const
	{
		return Begin;
	}

	int32 Num() const
	{
		return Size;
	}

	T* begin() const
	{
		return this->Begin;
	}

	T* end() const
	{
		return this->Begin + this->Size;
	}

private:
	T* Begin;
	int32 Size;
};

template <typename T>
struct TIsContiguousContainer< TArrayRange<T> >
{
	enum { Value = true };
};

/**
 * Stable merge to perform sort below. Stable sort is slower than non-stable
 * algorithm.
 *
 * @param Out Pointer to the first element of output array.
 * @param In Pointer to the first element to sort.
 * @param Mid Middle point of the table, i.e. merge separator.
 * @param Num Number of elements in the whole table.
 * @param Predicate Predicate class.
 */
template<class T, class PREDICATE_CLASS>
void Merge(T* Out, T* In, const int32 Mid, const int32 Num, const PREDICATE_CLASS& Predicate)
{
	int32 Merged = 0;
	int32 Picked;
	int32 A = 0, B = Mid;

	while (Merged < Num)
	{
		if (Merged != B && (B >= Num || !Predicate(In[B], In[A])))
		{
			Picked = A++;
		}
		else
		{
			Picked = B++;
		}

		Out[Merged] = In[Picked];

		++Merged;
	}
}

/**
 * Euclidean algorithm using modulo policy.
 */
class FEuclidDivisionGCD
{
public:
	/**
	 * Calculate GCD.
	 *
	 * @param A First parameter.
	 * @param B Second parameter.
	 *
	 * @returns Greatest common divisor of A and B.
	 */
	static int32 GCD(int32 A, int32 B)
	{
		while (B != 0)
		{
			int32 Temp = B;
			B = A % B;
			A = Temp;
		}

		return A;
	}
};

/**
 * Array rotation using juggling technique.
 *
 * @template_param TGCDPolicy Policy for calculating greatest common divisor.
 */
template <class TGCDPolicy>
class TJugglingRotation
{
public:
	/**
	 * Rotates array.
	 *
	 * @param First Pointer to the array.
	 * @param From Rotation starting point.
	 * @param To Rotation ending point.
	 * @param Amount Amount of steps to rotate.
	 */
	template <class T>
	static void Rotate(T* First, const int32 From, const int32 To, const int32 Amount)
	{
		if (Amount == 0)
		{
			return;
		}

		auto Num = To - From;
		auto GCD = TGCDPolicy::GCD(Num, Amount);
		auto CycleSize = Num / GCD;

		for (int32 Index = 0; Index < GCD; ++Index)
		{
			T BufferObject = MoveTemp(First[From + Index]);
			int32 IndexToFill = Index;

			for (int32 InCycleIndex = 0; InCycleIndex < CycleSize; ++InCycleIndex)
			{
				IndexToFill = (IndexToFill + Amount) % Num;
				Exchange(First[From + IndexToFill], BufferObject);
			}
		}
	}
};

/**
 * Merge policy for merge sort.
 *
 * @template_param TRotationPolicy Policy for array rotation algorithm.
 */
template <class TRotationPolicy>
class TRotationInPlaceMerge
{
public:
	/**
	 * Two sorted arrays merging function.
	 *
	 * @param First Pointer to array.
	 * @param Mid Middle point i.e. separation point of two arrays to merge.
	 * @param Num Number of elements in array.
	 * @param Predicate Predicate for comparison.
	 */
	template <class T, class PREDICATE_CLASS>
	static void Merge(T* First, const int32 Mid, const int32 Num, const PREDICATE_CLASS& Predicate)
	{
		int32 AStart = 0;
		int32 BStart = Mid;

		while (AStart < BStart && BStart < Num)
		{
			// Index after the last value == First[BStart]
			int32 NewAOffset = (int32)AlgoImpl::UpperBoundInternal(First + AStart, BStart - AStart, First[BStart], FIdentityFunctor(), Predicate);
			AStart += NewAOffset;

			if (AStart >= BStart) // done
				break;

			// Index of the first value == First[AStart]
			int32 NewBOffset = (int32)AlgoImpl::LowerBoundInternal(First + BStart, Num - BStart, First[AStart], FIdentityFunctor(), Predicate);
			TRotationPolicy::Rotate(First, AStart, BStart + NewBOffset, NewBOffset);
			BStart += NewBOffset;
			AStart += NewBOffset + 1;
		}
	}
};

/**
 * Merge sort class.
 *
 * @template_param TMergePolicy Merging policy.
 * @template_param MinMergeSubgroupSize Minimal size of the subgroup that should be merged.
 */
template <class TMergePolicy, int32 MinMergeSubgroupSize = 2>
class TMergeSort
{
public:
	/**
	 * Sort the array.
	 *
	 * @param First Pointer to the array.
	 * @param Num Number of elements in the array.
	 * @param Predicate Predicate for comparison.
	 */
	template<class T, class PREDICATE_CLASS>
	static void Sort(T* First, const int32 Num, const PREDICATE_CLASS& Predicate)
	{
		int32 SubgroupStart = 0;

		if (MinMergeSubgroupSize > 1)
		{
			if (MinMergeSubgroupSize > 2)
			{
				// First pass with simple bubble-sort.
				do
				{
					int32 GroupEnd = FPlatformMath::Min(SubgroupStart + MinMergeSubgroupSize, Num);
					do
					{
						for (int32 It = SubgroupStart; It < GroupEnd - 1; ++It)
						{
							if (Predicate(First[It + 1], First[It]))
							{
								Exchange(First[It], First[It + 1]);
							}
						}
						GroupEnd--;
					} while (GroupEnd - SubgroupStart > 1);

					SubgroupStart += MinMergeSubgroupSize;
				} while (SubgroupStart < Num);
			}
			else
			{
				for (int32 Subgroup = 0; Subgroup < Num; Subgroup += 2)
				{
					if (Subgroup + 1 < Num && Predicate(First[Subgroup + 1], First[Subgroup]))
					{
						Exchange(First[Subgroup], First[Subgroup + 1]);
					}
				}
			}
		}

		int32 SubgroupSize = MinMergeSubgroupSize;
		while (SubgroupSize < Num)
		{
			SubgroupStart = 0;
			do
			{
				TMergePolicy::Merge(
					First + SubgroupStart,
					SubgroupSize,
					FPlatformMath::Min(SubgroupSize << 1, Num - SubgroupStart),
					Predicate);
				SubgroupStart += SubgroupSize << 1;
			} while (SubgroupStart < Num);

			SubgroupSize <<= 1;
		}
	}
};

/**
 * Stable sort elements using user defined predicate class. The sort is stable,
 * meaning that the ordering of equal items is preserved, but it's slower than
 * non-stable algorithm.
 *
 * This is the internal sorting function used by StableSort overrides.
 *
 * @param	First	pointer to the first element to sort
 * @param	Num		the number of items to sort
 * @param Predicate predicate class
 */
template<class T, class PREDICATE_CLASS>
void StableSortInternal(T* First, const int32 Num, const PREDICATE_CLASS& Predicate)
{
	TMergeSort<TRotationInPlaceMerge<TJugglingRotation<FEuclidDivisionGCD> > >::Sort(First, Num, Predicate);
}

/**
 * Very fast 32bit radix sort.
 * SortKeyClass defines operator() that takes ValueType and returns a uint32. Sorting based on key.
 * No comparisons. Is stable.
 * Use a smaller CountType for smaller histograms.
 */
template< typename ValueType, typename CountType, class SortKeyClass >
void RadixSort32( ValueType* RESTRICT Dst, ValueType* RESTRICT Src, CountType Num, const SortKeyClass& SortKey )
{
	CountType Histograms[ 1024 + 2048 + 2048 ];
	CountType* RESTRICT Histogram0 = Histograms + 0;
	CountType* RESTRICT Histogram1 = Histogram0 + 1024;
	CountType* RESTRICT Histogram2 = Histogram1 + 2048;

	FMemory::Memzero( Histograms, sizeof( Histograms ) );

	{
		// Parallel histogram generation pass
		const ValueType* RESTRICT s = (const ValueType* RESTRICT)Src;
		for( CountType i = 0; i < Num; i++ )
		{
			uint32 Key = SortKey( s[i] );
			Histogram0[ ( Key >>  0 ) & 1023 ]++;
			Histogram1[ ( Key >> 10 ) & 2047 ]++;
			Histogram2[ ( Key >> 21 ) & 2047 ]++;
		}
	}
	{
		// Prefix sum
		// Set each histogram entry to the sum of entries preceding it
		CountType Sum0 = 0;
		CountType Sum1 = 0;
		CountType Sum2 = 0;
		for( CountType i = 0; i < 1024; i++ )
		{
			CountType t;
			t = Histogram0[i] + Sum0; Histogram0[i] = Sum0 - 1; Sum0 = t;
			t = Histogram1[i] + Sum1; Histogram1[i] = Sum1 - 1; Sum1 = t;
			t = Histogram2[i] + Sum2; Histogram2[i] = Sum2 - 1; Sum2 = t;
		}
		for( CountType i = 1024; i < 2048; i++ )
		{
			CountType t;
			t = Histogram1[i] + Sum1; Histogram1[i] = Sum1 - 1; Sum1 = t;
			t = Histogram2[i] + Sum2; Histogram2[i] = Sum2 - 1; Sum2 = t;
		}
	}
	{
		// Sort pass 1
		const ValueType* RESTRICT s = (const ValueType* RESTRICT)Src;
		ValueType* RESTRICT d = Dst;
		for( CountType i = 0; i < Num; i++ )
		{
			ValueType Value = s[i];
			uint32 Key = SortKey( Value );
			d[ ++Histogram0[ ( (Key >> 0) & 1023 ) ] ] = Value;
		}
	}
	{
		// Sort pass 2
		const ValueType* RESTRICT s = (const ValueType* RESTRICT)Dst;
		ValueType* RESTRICT d = Src;
		for( CountType i = 0; i < Num; i++ )
		{
			ValueType Value = s[i];
			uint32 Key = SortKey( Value );
			d[ ++Histogram1[ ( (Key >> 10) & 2047 ) ] ] = Value;
		}
	}
	{
		// Sort pass 3
		const ValueType* RESTRICT s = (const ValueType* RESTRICT)Src;
		ValueType* RESTRICT d = Dst;
		for( CountType i = 0; i < Num; i++ )
		{
			ValueType Value = s[i];
			uint32 Key = SortKey( Value );
			d[ ++Histogram2[ ( (Key >> 21) & 2047 ) ] ] = Value;
		}
	}
}


template< typename T >
struct TRadixSortKeyCastUint32
{
	UE_FORCEINLINE_HINT uint32 operator()( const T& Value ) const
	{
		return (uint32)Value;
	}
};

template< typename ValueType, typename CountType >
void RadixSort32( ValueType* RESTRICT Dst, ValueType* RESTRICT Src, CountType Num )
{
	RadixSort32( Dst, Src, Num, TRadixSortKeyCastUint32< ValueType >() );
}

// float cast to uint32 which maintains sorted order
// http://codercorner.com/RadixSortRevisited.htm
struct FRadixSortKeyFloat
{
	inline uint32 operator()( float Value ) const
	{
		union { float f; uint32 i; } v;
		v.f = Value;

		uint32 mask = -int32( v.i >> 31 ) | 0x80000000;
		return v.i ^ mask;
	}
};

template< typename CountType >
void RadixSort32( float* RESTRICT Dst, float* RESTRICT Src, CountType Num )
{
	RadixSort32( Dst, Src, Num, FRadixSortKeyFloat() );
}

enum class ERadixSortBufferState
{
	IsInitialized,
	IsUninitialized
};

/**
 * Very fast 64bit radix sort.
 * SortKeyClass defines operator() that takes ValueType and returns a uint32. Sorting based on key.
 * No comparisons. Is stable.
 * The default takes up 40k of stack space. Use a smaller CountType for smaller histograms and less stack space.
 * Buffer needs to be able to hold at least Num elements.
 */
template< ERadixSortBufferState BufferState, typename ValueType, typename CountType, class SortKeyClass >
void RadixSort64(ValueType* RESTRICT Array, ValueType* RESTRICT Buffer, CountType Num, const SortKeyClass& SortKey)
{
	CountType Histograms[(1024 * 2) + (2048 * 4)];
	CountType* RESTRICT Histogram0 = Histograms + 0;
	CountType* RESTRICT Histogram1 = Histogram0 + 1024;
	CountType* RESTRICT Histogram2 = Histogram1 + 1024;
	CountType* RESTRICT Histogram3 = Histogram2 + 2048;
	CountType* RESTRICT Histogram4 = Histogram3 + 2048;
	CountType* RESTRICT Histogram5 = Histogram4 + 2048;
	FMemory::Memzero(Histograms, sizeof(Histograms));

	{
		// Parallel histogram generation pass
		ValueType* RESTRICT s = Array;
		for (CountType i = 0; i < Num; i++)
		{
			uint64 Key = SortKey( *s++ );
			Histogram0[(Key >> 0) & 1023]++;
			Histogram1[(Key >> 10) & 1023]++;
			Histogram2[(Key >> 20) & 2047]++;
			Histogram3[(Key >> 31) & 2047]++;
			Histogram4[(Key >> 42) & 2047]++;
			Histogram5[(Key >> 53) & 2047]++;
		}
	}
	{
		// Prefix sum
		// Set each histogram entry to the sum of entries preceding it
		CountType Sum0 = 0;
		CountType Sum1 = 0;
		CountType Sum2 = 0;
		CountType Sum3 = 0;
		CountType Sum4 = 0;
		CountType Sum5 = 0;
		CountType i = 0;
		for (; i < 1024; i++)
		{
			CountType t0 = Histogram0[i] + Sum0; Histogram0[i] = Sum0; Sum0 = t0;
			CountType t1 = Histogram1[i] + Sum1; Histogram1[i] = Sum1; Sum1 = t1;
			CountType t2 = Histogram2[i] + Sum2; Histogram2[i] = Sum2; Sum2 = t2;
			CountType t3 = Histogram3[i] + Sum3; Histogram3[i] = Sum3; Sum3 = t3;
			CountType t4 = Histogram4[i] + Sum4; Histogram4[i] = Sum4; Sum4 = t4;
			CountType t5 = Histogram5[i] + Sum5; Histogram5[i] = Sum5; Sum5 = t5;
		}
		for (; i < 2048; i++)
		{
			CountType t2 = Histogram2[i] + Sum2; Histogram2[i] = Sum2; Sum2 = t2;
			CountType t3 = Histogram3[i] + Sum3; Histogram3[i] = Sum3; Sum3 = t3;
			CountType t4 = Histogram4[i] + Sum4; Histogram4[i] = Sum4; Sum4 = t4;
			CountType t5 = Histogram5[i] + Sum5; Histogram5[i] = Sum5; Sum5 = t5;
		}
	}
	{
		// Sort pass 1
		ValueType* RESTRICT Source = Array;
		ValueType* RESTRICT Destination = Buffer;
		for (CountType i = 0; i < Num; i++)
		{
			uint64 Value = SortKey(*Source);
			SIZE_T Index = Histogram0[((Value >> 0) & 1023)]++;
			if constexpr (BufferState == ERadixSortBufferState::IsInitialized)
			{
				Destination[Index] = MoveTemp(*Source++);
			}
			else
			{
				new(Destination + Index) ValueType(MoveTemp(*Source++));
			}
		}
	}
	{
		// Sort pass 2
		ValueType* RESTRICT Source = Buffer;
		ValueType* RESTRICT Destination = Array;
		for (CountType i = 0; i < Num; i++)
		{
			uint64 Value = SortKey(*Source);
			Destination[Histogram1[((Value >> 10) & 1023)]++] = MoveTemp(*Source++);
		}
	}
	{
		// Sort pass 3
		ValueType* RESTRICT Source = Array;
		ValueType* RESTRICT Destination = Buffer;
		for (CountType i = 0; i < Num; i++)
		{
			uint64 Value = SortKey(*Source);
			Destination[Histogram2[((Value >> 20) & 2047)]++] = MoveTemp(*Source++);
		}
	}
	{
		// Sort pass 4
		ValueType* RESTRICT Source = Buffer;
		ValueType* RESTRICT Destination = Array;
		for (CountType i = 0; i < Num; i++)
		{
			uint64 Value = SortKey(*Source);
			Destination[Histogram3[((Value >> 31) & 2047)]++] = MoveTemp(*Source++);
		}
	}
	{
		// Sort pass 5
		ValueType* RESTRICT Source = Array;
		ValueType* RESTRICT Destination = Buffer;
		for (CountType i = 0; i < Num; i++)
		{
			uint64 Value = SortKey(*Source);
			Destination[Histogram4[((Value >> 42) & 2047)]++] = MoveTemp(*Source++);
		}
	}
	{
		// Sort pass 6
		ValueType* RESTRICT Source = Buffer;
		ValueType* RESTRICT Destination = Array;
		for (CountType i = 0; i < Num; i++)
		{
			uint64 Value = SortKey(*Source);
			Destination[Histogram5[((Value >> 53) & 2047)]++] = MoveTemp(*Source++);
		}
	}
}

template< typename T >
struct TRadixSortKeyCastUint64
{
	UE_FORCEINLINE_HINT uint64 operator()(const T& Value) const
	{
		return (uint64)Value;
	}
};

template< ERadixSortBufferState BufferState, typename ValueType, typename CountType >
void RadixSort64(ValueType* RESTRICT Array, ValueType* RESTRICT Buffer, CountType Num)
{
	RadixSort64<BufferState>(Array, Buffer, Num, TRadixSortKeyCastUint64< ValueType >());
}

template< typename ValueType, typename CountType, class SortKeyClass >
void RadixSort64(ValueType* RESTRICT Array, CountType Num, const SortKeyClass& SortKey)
{
	TArray<ValueType> Buffer;
	Buffer.AddUninitialized(Num);
	RadixSort64<ERadixSortBufferState::IsUninitialized>(Array, Buffer.GetData(), Num, SortKey);
}

template< typename ValueType, typename CountType >
void RadixSort64(ValueType* RESTRICT Array, CountType Num)
{
	RadixSort64(Array, Num, TRadixSortKeyCastUint64< ValueType >());
}
