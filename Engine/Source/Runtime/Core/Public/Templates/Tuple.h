// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Concepts/NotCVRefTo.h"
#include "Templates/UnrealTemplate.h"
#include "Delegates/IntegerSequence.h"
#include "Templates/Invoke.h"
#include "Templates/TupleFwd.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/MemoryLayout.h"
#include "Templates/Requires.h"
#include "Templates/TypeHash.h"
#include <tuple>
#include <type_traits>
#include <utility>

// MSVC generates bad codegen when getting a reference element from an rvalue tuple.  So, if the element type is
// an rvalue reference, get it via an lvalue - it will be the same reference value in either case.
//
// The bug is reported here: https://developercommunity.visualstudio.com/t/Bad-codegen-when-a-function-returns-an-a/10984053
#if defined(_MSC_VER) && !defined(__clang__)
	#define UE_TUPLE_REFERENCE_WORKAROUND 1
#else
	#define UE_TUPLE_REFERENCE_WORKAROUND 0
#endif

class FArchive;

template <typename T> constexpr bool TIsTuple_V = false;

template <typename... Types> constexpr bool TIsTuple_V<               TTuple<Types...>> = true;
template <typename... Types> constexpr bool TIsTuple_V<const          TTuple<Types...>> = true;
template <typename... Types> constexpr bool TIsTuple_V<      volatile TTuple<Types...>> = true;
template <typename... Types> constexpr bool TIsTuple_V<const volatile TTuple<Types...>> = true;

template <typename T>
struct TIsTuple
{
	enum { Value = TIsTuple_V<T> };
};

template <typename... Types>
constexpr TTuple<std::decay_t<Types>...> MakeTuple(Types&&... Args);

namespace UE::Core::Private::Tuple
{
	enum EForwardingConstructor { ForwardingConstructor };
	enum EOtherTupleConstructor { OtherTupleConstructor };

	template <typename T, typename... Types>
	constexpr uint32 TTypeCountInParameterPack_V = 0;

	template <typename T, typename U, typename... Types>
	constexpr uint32 TTypeCountInParameterPack_V<T, U, Types...> = TTypeCountInParameterPack_V<T, Types...> + (std::is_same_v<T, U> ? 1 : 0);

	template <typename T, uint32 Index, uint32 TupleSize>
	struct TTupleBaseElement
	{
		template <typename ArgType>
		constexpr explicit TTupleBaseElement(EForwardingConstructor, ArgType&& Arg)
			: Value(Forward<ArgType>(Arg))
		{
		}

		constexpr TTupleBaseElement()
			: Value()
		{
		}

		TTupleBaseElement(TTupleBaseElement&&) = default;
		TTupleBaseElement(const TTupleBaseElement&) = default;
		TTupleBaseElement& operator=(TTupleBaseElement&&) = default;
		TTupleBaseElement& operator=(const TTupleBaseElement&) = default;

		T Value;
	};

	template <typename T>
	struct TTupleBaseElement<T, 0, 2>
	{
		template <typename ArgType>
		constexpr explicit TTupleBaseElement(EForwardingConstructor, ArgType&& Arg)
			: Key(Forward<ArgType>(Arg))
		{
		}

		constexpr TTupleBaseElement()
			: Key()
		{
		}

		TTupleBaseElement(TTupleBaseElement&&) = default;
		TTupleBaseElement(const TTupleBaseElement&) = default;
		TTupleBaseElement& operator=(TTupleBaseElement&&) = default;
		TTupleBaseElement& operator=(const TTupleBaseElement&) = default;

		T Key;
	};

	template <uint32 Index, uint32 TupleSize>
	struct TTupleElementGetterByIndex
	{
		template <typename DeducedType, typename TupleType>
		static inline decltype(auto) GetImpl(const volatile TTupleBaseElement<DeducedType, Index, TupleSize>& Element, TupleType&& Tuple)
		{
#if UE_TUPLE_REFERENCE_WORKAROUND
			if constexpr (!std::is_reference_v<TupleType> && std::is_reference_v<DeducedType>)
			{
				return (DeducedType)GetImpl(Element, Tuple);
			}
			else
#endif
		{
			// Brackets are important here - we want a reference type to be returned, not object type
			decltype(auto) Result = (ForwardAsBase<TupleType, TTupleBaseElement<DeducedType, Index, TupleSize>>(Tuple).Value);

			// Keep tuple rvalue references to rvalue reference elements as rvalues, because that's how std::get() works, not how C++ struct member access works.
			return static_cast<std::conditional_t<!std::is_reference_v<TupleType> && std::is_rvalue_reference_v<DeducedType>, DeducedType, decltype(Result)>>(Result);
		}
		}

		template <typename TupleType>
		static UE_FORCEINLINE_HINT decltype(auto) Get(TupleType&& Tuple)
		{
			return GetImpl(Tuple, Forward<TupleType>(Tuple));
		}
	};

	template <>
	struct TTupleElementGetterByIndex<0, 2>
	{
		template <typename TupleType>
		static inline decltype(auto) Get(TupleType&& Tuple)
		{
#if UE_TUPLE_REFERENCE_WORKAROUND
			using KeyType = decltype(std::decay_t<TupleType>::Key);
			if constexpr (!std::is_reference_v<TupleType> && std::is_reference_v<KeyType>)
		{
				return (KeyType)Get(Tuple);
		}
			else
#endif
		{
				// Brackets are important here - we want a reference type to be returned, not object type
				decltype(auto) Result = (ForwardAsBase<TupleType, TTupleBaseElement<decltype(Tuple.Key), 0, 2>>(Tuple).Key);

				// Keep tuple rvalue references to rvalue reference elements as rvalues, because that's how std::get() works, not how C++ struct member access works.
				return static_cast<std::conditional_t<!std::is_reference_v<TupleType> && std::is_rvalue_reference_v<decltype(Tuple.Key)>, decltype(Tuple.Key), decltype(Result)>>(Result);
			}
		}
	};

	template <typename Type, uint32 TupleSize>
	struct TTupleElementGetterByType
	{
		template <uint32 DeducedIndex, typename TupleType>
		static UE_FORCEINLINE_HINT decltype(auto) GetImpl(const volatile TTupleBaseElement<Type, DeducedIndex, TupleSize>&, TupleType&& Tuple)
		{
			return TTupleElementGetterByIndex<DeducedIndex, TupleSize>::Get(Forward<TupleType>(Tuple));
		}

		template <typename TupleType>
		static UE_FORCEINLINE_HINT decltype(auto) Get(TupleType&& Tuple)
		{
			return GetImpl(Tuple, Forward<TupleType>(Tuple));
		}
	};

	template <uint32 ArgCount, uint32 ArgToCompare>
	struct FEqualityHelper
	{
		template <typename TupleType>
		UE_FORCEINLINE_HINT static bool Compare(const TupleType& Lhs, const TupleType& Rhs)
		{
			return Lhs.template Get<ArgToCompare>() == Rhs.template Get<ArgToCompare>() && FEqualityHelper<ArgCount, ArgToCompare + 1>::Compare(Lhs, Rhs);
		}
	};

	template <uint32 ArgCount>
	struct FEqualityHelper<ArgCount, ArgCount>
	{
		template <typename TupleType>
		UE_FORCEINLINE_HINT static bool Compare(const TupleType& Lhs, const TupleType& Rhs)
		{
			return true;
		}
	};

	template <uint32 NumArgs, uint32 ArgToCompare = 0, bool Last = ArgToCompare + 1 == NumArgs>
	struct TLessThanHelper
	{
		template <typename TupleType>
		UE_FORCEINLINE_HINT static bool Do(const TupleType& Lhs, const TupleType& Rhs)
		{
			return Lhs.template Get<ArgToCompare>() < Rhs.template Get<ArgToCompare>() || (!(Rhs.template Get<ArgToCompare>() < Lhs.template Get<ArgToCompare>()) && TLessThanHelper<NumArgs, ArgToCompare + 1>::Do(Lhs, Rhs));
		}
	};

	template <uint32 NumArgs, uint32 ArgToCompare>
	struct TLessThanHelper<NumArgs, ArgToCompare, true>
	{
		template <typename TupleType>
		UE_FORCEINLINE_HINT static bool Do(const TupleType& Lhs, const TupleType& Rhs)
		{
			return Lhs.template Get<ArgToCompare>() < Rhs.template Get<ArgToCompare>();
		}
	};

	template <uint32 NumArgs>
	struct TLessThanHelper<NumArgs, NumArgs, false>
	{
		template <typename TupleType>
		UE_FORCEINLINE_HINT static bool Do(const TupleType& Lhs, const TupleType& Rhs)
		{
			return false;
		}
	};

	template <typename Indices, typename... Types>
	struct TTupleBase;

	template <uint32... Indices, typename... Types>
	struct TTupleBase<TIntegerSequence<uint32, Indices...>, Types...> : TTupleBaseElement<Types, Indices, sizeof...(Types)>...
	{
		template <typename... ArgTypes>
		constexpr explicit TTupleBase(EForwardingConstructor, ArgTypes&&... Args)
			: TTupleBaseElement<Types, Indices, sizeof...(Types)>(ForwardingConstructor, Forward<ArgTypes>(Args))...
		{
		}

		template <typename TupleType>
		explicit TTupleBase(EOtherTupleConstructor, TupleType&& Other)
			: TTupleBaseElement<Types, Indices, sizeof...(Types)>(ForwardingConstructor, Forward<TupleType>(Other).template Get<Indices>())...
		{
		}

		constexpr TTupleBase() = default;
		TTupleBase(TTupleBase&& Other) = default;
		TTupleBase(const TTupleBase& Other) = default;
		TTupleBase& operator=(TTupleBase&& Other) = default;
		TTupleBase& operator=(const TTupleBase& Other) = default;

		template <uint32 Index> UE_FORCEINLINE_HINT decltype(auto) Get()               &  { static_assert(Index < sizeof...(Types), "Invalid index passed to TTuple::Get"); return TTupleElementGetterByIndex<Index, sizeof...(Types)>::Get(static_cast<               TTupleBase& >(*this)); }
		template <uint32 Index> UE_FORCEINLINE_HINT decltype(auto) Get() const         &  { static_assert(Index < sizeof...(Types), "Invalid index passed to TTuple::Get"); return TTupleElementGetterByIndex<Index, sizeof...(Types)>::Get(static_cast<const          TTupleBase& >(*this)); }
		template <uint32 Index> UE_FORCEINLINE_HINT decltype(auto) Get()       volatile&  { static_assert(Index < sizeof...(Types), "Invalid index passed to TTuple::Get"); return TTupleElementGetterByIndex<Index, sizeof...(Types)>::Get(static_cast<      volatile TTupleBase& >(*this)); }
		template <uint32 Index> UE_FORCEINLINE_HINT decltype(auto) Get() const volatile&  { static_assert(Index < sizeof...(Types), "Invalid index passed to TTuple::Get"); return TTupleElementGetterByIndex<Index, sizeof...(Types)>::Get(static_cast<const volatile TTupleBase& >(*this)); }
		template <uint32 Index> UE_FORCEINLINE_HINT decltype(auto) Get()               && { static_assert(Index < sizeof...(Types), "Invalid index passed to TTuple::Get"); return TTupleElementGetterByIndex<Index, sizeof...(Types)>::Get(static_cast<               TTupleBase&&>(*this)); }
		template <uint32 Index> UE_FORCEINLINE_HINT decltype(auto) Get() const         && { static_assert(Index < sizeof...(Types), "Invalid index passed to TTuple::Get"); return TTupleElementGetterByIndex<Index, sizeof...(Types)>::Get(static_cast<const          TTupleBase&&>(*this)); }
		template <uint32 Index> UE_FORCEINLINE_HINT decltype(auto) Get()       volatile&& { static_assert(Index < sizeof...(Types), "Invalid index passed to TTuple::Get"); return TTupleElementGetterByIndex<Index, sizeof...(Types)>::Get(static_cast<      volatile TTupleBase&&>(*this)); }
		template <uint32 Index> UE_FORCEINLINE_HINT decltype(auto) Get() const volatile&& { static_assert(Index < sizeof...(Types), "Invalid index passed to TTuple::Get"); return TTupleElementGetterByIndex<Index, sizeof...(Types)>::Get(static_cast<const volatile TTupleBase&&>(*this)); }

		template <typename T> UE_FORCEINLINE_HINT decltype(auto) Get()               &  { static_assert(TTypeCountInParameterPack_V<T, Types...> == 1, "Invalid type passed to TTuple::Get"); return TTupleElementGetterByType<T, sizeof...(Types)>::Get(static_cast<               TTupleBase& >(*this)); }
		template <typename T> UE_FORCEINLINE_HINT decltype(auto) Get() const         &  { static_assert(TTypeCountInParameterPack_V<T, Types...> == 1, "Invalid type passed to TTuple::Get"); return TTupleElementGetterByType<T, sizeof...(Types)>::Get(static_cast<const          TTupleBase& >(*this)); }
		template <typename T> UE_FORCEINLINE_HINT decltype(auto) Get()       volatile&  { static_assert(TTypeCountInParameterPack_V<T, Types...> == 1, "Invalid type passed to TTuple::Get"); return TTupleElementGetterByType<T, sizeof...(Types)>::Get(static_cast<      volatile TTupleBase& >(*this)); }
		template <typename T> UE_FORCEINLINE_HINT decltype(auto) Get() const volatile&  { static_assert(TTypeCountInParameterPack_V<T, Types...> == 1, "Invalid type passed to TTuple::Get"); return TTupleElementGetterByType<T, sizeof...(Types)>::Get(static_cast<const volatile TTupleBase& >(*this)); }
		template <typename T> UE_FORCEINLINE_HINT decltype(auto) Get()               && { static_assert(TTypeCountInParameterPack_V<T, Types...> == 1, "Invalid type passed to TTuple::Get"); return TTupleElementGetterByType<T, sizeof...(Types)>::Get(static_cast<               TTupleBase&&>(*this)); }
		template <typename T> UE_FORCEINLINE_HINT decltype(auto) Get() const         && { static_assert(TTypeCountInParameterPack_V<T, Types...> == 1, "Invalid type passed to TTuple::Get"); return TTupleElementGetterByType<T, sizeof...(Types)>::Get(static_cast<const          TTupleBase&&>(*this)); }
		template <typename T> UE_FORCEINLINE_HINT decltype(auto) Get()       volatile&& { static_assert(TTypeCountInParameterPack_V<T, Types...> == 1, "Invalid type passed to TTuple::Get"); return TTupleElementGetterByType<T, sizeof...(Types)>::Get(static_cast<      volatile TTupleBase&&>(*this)); }
		template <typename T> UE_FORCEINLINE_HINT decltype(auto) Get() const volatile&& { static_assert(TTypeCountInParameterPack_V<T, Types...> == 1, "Invalid type passed to TTuple::Get"); return TTupleElementGetterByType<T, sizeof...(Types)>::Get(static_cast<const volatile TTupleBase&&>(*this)); }

		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyAfter(FuncType&& Func, ArgTypes&&... Args)               &  { return ::Invoke(Func, Forward<ArgTypes>(Args)..., static_cast<               TTupleBase& >(*this).template Get<Indices>()...); }
		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyAfter(FuncType&& Func, ArgTypes&&... Args) const         &  { return ::Invoke(Func, Forward<ArgTypes>(Args)..., static_cast<const          TTupleBase& >(*this).template Get<Indices>()...); }
		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyAfter(FuncType&& Func, ArgTypes&&... Args)       volatile&  { return ::Invoke(Func, Forward<ArgTypes>(Args)..., static_cast<      volatile TTupleBase& >(*this).template Get<Indices>()...); }
		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyAfter(FuncType&& Func, ArgTypes&&... Args) const volatile&  { return ::Invoke(Func, Forward<ArgTypes>(Args)..., static_cast<const volatile TTupleBase& >(*this).template Get<Indices>()...); }
		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyAfter(FuncType&& Func, ArgTypes&&... Args)               && { return ::Invoke(Func, Forward<ArgTypes>(Args)..., static_cast<               TTupleBase&&>(*this).template Get<Indices>()...); }
		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyAfter(FuncType&& Func, ArgTypes&&... Args) const         && { return ::Invoke(Func, Forward<ArgTypes>(Args)..., static_cast<const          TTupleBase&&>(*this).template Get<Indices>()...); }
		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyAfter(FuncType&& Func, ArgTypes&&... Args)       volatile&& { return ::Invoke(Func, Forward<ArgTypes>(Args)..., static_cast<      volatile TTupleBase&&>(*this).template Get<Indices>()...); }
		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyAfter(FuncType&& Func, ArgTypes&&... Args) const volatile&& { return ::Invoke(Func, Forward<ArgTypes>(Args)..., static_cast<const volatile TTupleBase&&>(*this).template Get<Indices>()...); }

		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyBefore(FuncType&& Func, ArgTypes&&... Args)               &  { return ::Invoke(Func, static_cast<               TTupleBase& >(*this).template Get<Indices>()..., Forward<ArgTypes>(Args)...); }
		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyBefore(FuncType&& Func, ArgTypes&&... Args) const         &  { return ::Invoke(Func, static_cast<const          TTupleBase& >(*this).template Get<Indices>()..., Forward<ArgTypes>(Args)...); }
		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyBefore(FuncType&& Func, ArgTypes&&... Args)       volatile&  { return ::Invoke(Func, static_cast<      volatile TTupleBase& >(*this).template Get<Indices>()..., Forward<ArgTypes>(Args)...); }
		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyBefore(FuncType&& Func, ArgTypes&&... Args) const volatile&  { return ::Invoke(Func, static_cast<const volatile TTupleBase& >(*this).template Get<Indices>()..., Forward<ArgTypes>(Args)...); }
		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyBefore(FuncType&& Func, ArgTypes&&... Args)               && { return ::Invoke(Func, static_cast<               TTupleBase&&>(*this).template Get<Indices>()..., Forward<ArgTypes>(Args)...); }
		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyBefore(FuncType&& Func, ArgTypes&&... Args) const         && { return ::Invoke(Func, static_cast<const          TTupleBase&&>(*this).template Get<Indices>()..., Forward<ArgTypes>(Args)...); }
		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyBefore(FuncType&& Func, ArgTypes&&... Args)       volatile&& { return ::Invoke(Func, static_cast<      volatile TTupleBase&&>(*this).template Get<Indices>()..., Forward<ArgTypes>(Args)...); }
		template <typename FuncType, typename... ArgTypes> decltype(auto) ApplyBefore(FuncType&& Func, ArgTypes&&... Args) const volatile&& { return ::Invoke(Func, static_cast<const volatile TTupleBase&&>(*this).template Get<Indices>()..., Forward<ArgTypes>(Args)...); }

		inline static void Serialize(FArchive& Ar, TTupleBase& Tuple)
		{
			// This should be implemented with a fold expression when our compilers support it
			int Temp[] = { 0, (Ar << Tuple.template Get<Indices>(), 0)... };
			(void)Temp;
		}

		inline static void SerializeStructured(FStructuredArchive::FSlot Slot, TTupleBase& Tuple)
		{
			// This should be implemented with a fold expression when our compilers support it
			FStructuredArchive::FStream Stream = Slot.EnterStream();
			int Temp[] = { 0, (Stream.EnterElement() << Tuple.template Get<Indices>(), 0)... };
			(void)Temp;
		}

		UE_FORCEINLINE_HINT bool operator==(const TTupleBase& Rhs) const
		{
			// This could be implemented with a fold expression when our compilers support it
			return FEqualityHelper<sizeof...(Types), 0>::Compare(*this, Rhs);
		}

		UE_FORCEINLINE_HINT bool operator!=(const TTupleBase& Rhs) const
		{
			return !(*this == Rhs);
		}

		UE_FORCEINLINE_HINT bool operator<(const TTupleBase& Rhs) const
		{
			return TLessThanHelper<sizeof...(Types)>::Do(*this, Rhs);
		}

		UE_FORCEINLINE_HINT bool operator<=(const TTupleBase& Rhs) const
		{
			return !(Rhs < *this);
		}

		UE_FORCEINLINE_HINT bool operator>(const TTupleBase& Rhs) const
		{
			return Rhs < *this;
		}

		UE_FORCEINLINE_HINT bool operator>=(const TTupleBase& Rhs) const
		{
			return !(*this < Rhs);
		}
	};

	template <typename LhsType, typename RhsType, uint32... Indices>
	static void Assign(LhsType& Lhs, RhsType&& Rhs, TIntegerSequence<uint32, Indices...>)
	{
		// This should be implemented with a fold expression when our compilers support it
		int Temp[] = { 0, (Lhs.template Get<Indices>() = Forward<RhsType>(Rhs).template Get<Indices>(), 0)... };
		(void)Temp;
	}

	template <typename... ElementTypes, typename... Types>
	UE_FORCEINLINE_HINT constexpr TTuple<ElementTypes...> MakeTupleImpl(Types&&... Args)
	{
		return TTuple<ElementTypes...>(Forward<Types>(Args)...);
	}

	template <typename IntegerSequence>
	struct TTransformTuple_Impl;

	template <uint32... Indices>
	struct TTransformTuple_Impl<TIntegerSequence<uint32, Indices...>>
	{
		template <typename TupleType, typename FuncType>
		static decltype(auto) Do(TupleType&& Tuple, FuncType Func)
		{
			return MakeTuple(Func(Forward<TupleType>(Tuple).template Get<Indices>())...);
		}
	};

	template <typename IntegerSequence>
	struct TVisitTupleElements_Impl;

	template <uint32... Indices>
	struct TVisitTupleElements_Impl<TIntegerSequence<uint32, Indices...>>
	{
		// We need a second function to do the invocation for a particular index, to avoid the pack expansion being
		// attempted on the indices and tuples simultaneously.
		template <uint32 Index, typename FuncType, typename... TupleTypes>
		UE_FORCEINLINE_HINT static void InvokeFunc(FuncType&& Func, TupleTypes&&... Tuples)
		{
			::Invoke(Forward<FuncType>(Func), Forward<TupleTypes>(Tuples).template Get<Index>()...);
		}

		template <typename FuncType, typename... TupleTypes>
		static void Do(FuncType&& Func, TupleTypes&&... Tuples)
		{
			// This should be implemented with a fold expression when our compilers support it
			int Temp[] = { 0, (InvokeFunc<Indices>(Forward<FuncType>(Func), Forward<TupleTypes>(Tuples)...), 0)... };
			(void)Temp;
		}
	};


	template <typename TupleType>
	struct TCVTupleArity;

	template <typename... Types>
	struct TCVTupleArity<const volatile TTuple<Types...>>
	{
		enum { Value = sizeof...(Types) };
	};

	template <typename TupleType, uint32 Num>
	struct TTuplePopFront
	{
	};

	template <typename Type0, typename... Types, uint32 Num>
	struct TTuplePopFront<TTuple<Type0, Types...>, Num> : TTuplePopFront<TTuple<Types...>, Num - 1>
	{
	};

	template <typename Type0, typename... Types>
	struct TTuplePopFront<TTuple<Type0, Types...>, 0>
	{
		using Type = TTuple<Type0, Types...>;
	};

	template <>
	struct TTuplePopFront<TTuple<>, 0>
	{
		using Type = TTuple<>;
	};

	template <typename TupleType, uint32 Num> struct TTuplePopFront<const          TupleType, Num> { using Type = const          typename TTuplePopFront<TupleType, Num>::Type; };
	template <typename TupleType, uint32 Num> struct TTuplePopFront<      volatile TupleType, Num> { using Type =       volatile typename TTuplePopFront<TupleType, Num>::Type; };
	template <typename TupleType, uint32 Num> struct TTuplePopFront<const volatile TupleType, Num> { using Type = const volatile typename TTuplePopFront<TupleType, Num>::Type; };

	template <typename TupleType, uint32 Num, typename ResultTuple = TTuple<>>
	struct TTuplePopBack
	{
	};

	template <typename Type0, typename... Types, uint32 Num, typename... ResultTypes>
	struct TTuplePopBack<TTuple<Type0, Types...>, Num, TTuple<ResultTypes...>> : TTuplePopBack<TTuple<Types...>, Num - 1, TTuple<ResultTypes..., Type0>>
	{
	};

	template <typename Type0, typename... Types, typename... ResultTypes>
	struct TTuplePopBack<TTuple<Type0, Types...>, 0, TTuple<ResultTypes...>>
	{
		using Type = TTuple<ResultTypes...>;
	};

	template <typename... ResultTypes>
	struct TTuplePopBack<TTuple<>, 0, TTuple<ResultTypes...>>
	{
		using Type = TTuple<ResultTypes...>;
	};

	template <typename TupleType, uint32 Num> struct TTuplePopBack<const          TupleType, Num> { using Type = const          typename TTuplePopBack<TupleType, Num>::Type; };
	template <typename TupleType, uint32 Num> struct TTuplePopBack<      volatile TupleType, Num> { using Type =       volatile typename TTuplePopBack<TupleType, Num>::Type; };
	template <typename TupleType, uint32 Num> struct TTuplePopBack<const volatile TupleType, Num> { using Type = const volatile typename TTuplePopBack<TupleType, Num>::Type; };

	template <typename Type, typename TupleType>
	struct TCVTupleIndex
	{
		static_assert(sizeof(TupleType) == 0, "TTupleIndex instantiated with a non-tuple type");
		static constexpr uint32 Value = 0;
	};

	template <typename Type, typename... TupleTypes>
	struct TCVTupleIndex<Type, const volatile TTuple<TupleTypes...>>
	{
		static_assert(TTypeCountInParameterPack_V<Type, TupleTypes...> >= 1, "TTupleIndex instantiated with a tuple which does not contain the type");
		static_assert(TTypeCountInParameterPack_V<Type, TupleTypes...> <= 1, "TTupleIndex instantiated with a tuple which contains multiple instances of the type");

	private:
		template <uint32 DeducedIndex>
		static auto Resolve(TTupleBaseElement<Type, DeducedIndex, sizeof...(TupleTypes)>*) -> char(&)[DeducedIndex + 1];
		static auto Resolve(...) -> char;

	public:
		static constexpr uint32 Value = sizeof(Resolve(DeclVal<TTuple<TupleTypes...>*>())) - 1;
	};

	template <uint32 Index, typename TupleType>
	struct TCVTupleElement
	{
		static_assert(sizeof(TupleType) == 0, "TTupleElement instantiated with a non-tuple type");
		using Type = void;
	};

	template <uint32 Index, typename... TupleTypes>
	struct TCVTupleElement<Index, const volatile TTuple<TupleTypes...>>
	{
		static_assert(Index < sizeof...(TupleTypes), "TTupleElement instantiated with an invalid index");

#ifdef __clang__
		using Type = __type_pack_element<Index, TupleTypes...>;
#else
	private:
		template <typename DeducedType>
		static DeducedType Resolve(TTupleBaseElement<DeducedType, Index, sizeof...(TupleTypes)>*);
		static void Resolve(...);

	public:
		using Type = decltype(Resolve(DeclVal<TTuple<TupleTypes...>*>()));
#endif
	};

	template <uint32 ArgToCombine, uint32 ArgCount>
	struct TGetTupleHashHelper
	{
		template <typename TupleType>
		UE_FORCEINLINE_HINT static uint32 Do(uint32 Hash, const TupleType& Tuple)
		{
			return TGetTupleHashHelper<ArgToCombine + 1, ArgCount>::Do(HashCombineFast(Hash, GetTypeHashHelper(Tuple.template Get<ArgToCombine>())), Tuple);
		}
	};

	template <uint32 ArgIndex>
	struct TGetTupleHashHelper<ArgIndex, ArgIndex>
	{
		template <typename TupleType>
		UE_FORCEINLINE_HINT static uint32 Do(uint32 Hash, const TupleType& Tuple)
		{
			return Hash;
		}
	};

	template <uint32 Index>
	struct TGetElementByIndex
	{
		template <typename TupleType>
			requires (TIsTuple_V<std::decay_t<TupleType>>)
		constexpr auto operator()(TupleType&& Tuple) const -> decltype(((TupleType&&)Tuple).template Get<Index>())
		{
			return ((TupleType&&)Tuple).template Get<Index>();
		}
	};

	template <typename Type>
	struct TGetElementByType
	{
		template <typename TupleType>
			requires (TIsTuple_V<std::decay_t<TupleType>>)
		constexpr auto operator()(TupleType&& Tuple) const -> decltype(((TupleType&&)Tuple).template Get<Type>())
		{
			return ((TupleType&&)Tuple).template Get<Type>();
		}
	};
}

template <typename... Types>
struct TTuple : UE::Core::Private::Tuple::TTupleBase<TMakeIntegerSequence<uint32, sizeof...(Types)>, Types...>
{
private:
	typedef UE::Core::Private::Tuple::TTupleBase<TMakeIntegerSequence<uint32, sizeof...(Types)>, Types...> Super;

public:
	template <UE::CNotCVRefTo<TTuple>... ArgTypes>
		requires (std::is_constructible_v<Types, ArgTypes&&> && ...)
	constexpr explicit(!std::conjunction_v<std::is_convertible<ArgTypes&&, Types>...>) TTuple(ArgTypes&&... Args)
		: Super(UE::Core::Private::Tuple::ForwardingConstructor, Forward<ArgTypes>(Args)...)
	{
	}

	template <typename... OtherTypes>
		requires (std::is_constructible_v<Types, OtherTypes&&> && ...)
	TTuple(TTuple<OtherTypes...>&& Other)
		: Super(UE::Core::Private::Tuple::OtherTupleConstructor, MoveTemp(Other))
	{
	}

	template <typename... OtherTypes>
		requires (std::is_constructible_v<Types, const OtherTypes&> && ...)
	TTuple(const TTuple<OtherTypes...>& Other)
		: Super(UE::Core::Private::Tuple::OtherTupleConstructor, Other)
	{
	}

	constexpr TTuple() requires (std::is_default_constructible_v<Types> && ...) = default;
	TTuple(TTuple&&) = default;
	TTuple(const TTuple&) = default;
	TTuple& operator=(TTuple&&) = default;
	TTuple& operator=(const TTuple&) = default;

	template <typename... OtherTypes>
		requires (std::is_assignable_v<Types, const OtherTypes&> && ...)
	TTuple& operator=(const TTuple<OtherTypes...>& Other)
	{
		UE::Core::Private::Tuple::Assign(*this, Other, TMakeIntegerSequence<uint32, sizeof...(Types)>{});
		return *this;
	}

	template <typename... OtherTypes>
		requires (std::is_assignable_v<Types, OtherTypes&&> && ...)
	TTuple& operator=(TTuple<OtherTypes...>&& Other)
	{
		UE::Core::Private::Tuple::Assign(*this, MoveTemp(Other), TMakeIntegerSequence<uint32, sizeof...(OtherTypes)>{});
		return *this;
	}
};

template <typename... Types>
UE_FORCEINLINE_HINT uint32 GetTypeHash(const TTuple<Types...>& Tuple)
{
	return UE::Core::Private::Tuple::TGetTupleHashHelper<1u, sizeof...(Types)>::Do(GetTypeHash(Tuple.template Get<0>()), Tuple);
}

UE_FORCEINLINE_HINT uint32 GetTypeHash(const TTuple<>& Tuple)
{
	return 0;
}

namespace Freeze
{
	template<typename KeyType, typename ValueType>
	void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const TTuple<KeyType, ValueType>& Object, const FTypeLayoutDesc&)
	{
		Writer.WriteObject(Object.Key);
		Writer.WriteObject(Object.Value);
	}

	template<typename KeyType, typename ValueType>
	uint32 IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const TTuple<KeyType, ValueType>& Object, void* OutDst)
	{
		TTuple<KeyType, ValueType>* DstObject = (TTuple<KeyType, ValueType>*)OutDst;
		Context.UnfreezeObject(Object.Key, &DstObject->Key);
		Context.UnfreezeObject(Object.Value, &DstObject->Value);
		return sizeof(Object);
	}

	template<typename KeyType, typename ValueType>
	uint32 IntrinsicAppendHash(const TTuple<KeyType, ValueType>* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher)
	{
		return Freeze::AppendHashPair(StaticGetTypeLayoutDesc<KeyType>(), StaticGetTypeLayoutDesc<ValueType>(), LayoutParams, Hasher);
	}

	template<typename KeyType, typename ValueType>
	uint32 IntrinsicGetTargetAlignment(const TTuple<KeyType, ValueType>* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams)
	{
		const uint32 KeyAlignment = GetTargetAlignment(StaticGetTypeLayoutDesc<KeyType>(), LayoutParams);
		const uint32 ValueAlignment = GetTargetAlignment(StaticGetTypeLayoutDesc<ValueType>(), LayoutParams);
		return FMath::Min(FMath::Max(KeyAlignment, ValueAlignment), LayoutParams.MaxFieldAlignment);
	}
}

DECLARE_TEMPLATE_INTRINSIC_TYPE_LAYOUT((template <typename KeyType, typename ValueType>), (TTuple<KeyType, ValueType>));

/**
 * Traits class which calculates the number of elements in a tuple.
 */
template <typename TupleType>
struct TTupleArity : UE::Core::Private::Tuple::TCVTupleArity<const volatile TupleType>
{
};


/**
 * Traits class which pops a number of element types from the start of a tuple type.
 *
 * Example: TTuplePopFront<TTuple<A, B, C, D, E>, 2>::Type == TTuple<C, D, E>
 */
template <typename TupleType, uint32 Num = 1>
struct TTuplePopFront : UE::Core::Private::Tuple::TTuplePopFront<TupleType, Num>
{
};

template <typename TupleType, uint32 Num = 1>
using TTuplePopFront_T = typename TTuplePopFront<TupleType, Num>::Type;


/**
 * Traits class which pops a number of element types from the end of a tuple type.
 *
 * Example: TTuplePopBack<TTuple<A, B, C, D, E>, 2>::Type == TTuple<A, B, C>
 */
template <typename TupleType, uint32 Num = 1>
struct TTuplePopBack : UE::Core::Private::Tuple::TTuplePopBack<TupleType, TTupleArity<TupleType>::Value - Num>
{
};

template <typename TupleType, uint32 Num = 1>
using TTuplePopBack_T = typename TTuplePopBack<TupleType, Num>::Type;


/**
 * Traits class which gets the tuple index of a given type from a given TTuple.
 * If the type doesn't appear, or appears more than once, a compile error is generated.
 *
 * Given Type = char, and Tuple = TTuple<int, float, char>,
 * TTupleIndex<Type, Tuple>::Value will be 2.
 */
template <typename Type, typename TupleType>
using TTupleIndex = UE::Core::Private::Tuple::TCVTupleIndex<Type, const volatile TupleType>;


/**
 * Traits class which gets the element type of a TTuple with the given index.
 * If the index is out of range, a compile error is generated.
 *
 * Given Index = 1, and Tuple = TTuple<int, float, char>,
 * TTupleElement<Index, Tuple>::Type will be float.
 */
template <uint32 Index, typename TupleType>
using TTupleElement = UE::Core::Private::Tuple::TCVTupleElement<Index, const volatile TupleType>;


/**
 * Makes a TTuple from some arguments.  The type of the TTuple elements are the decayed versions of the arguments.
 *
 * @param  Args  The arguments used to construct the tuple.
 * @return A tuple containing a copy of the arguments.
 *
 * Example:
 *
 * void Func(const int32 A, FString&& B)
 * {
 *     // Equivalent to:
 *     // TTuple<int32, const TCHAR*, FString> MyTuple(A, TEXT("Hello"), MoveTemp(B));
 *     auto MyTuple = MakeTuple(A, TEXT("Hello"), MoveTemp(B));
 * }
 */
template <typename... Types>
UE_FORCEINLINE_HINT constexpr TTuple<std::decay_t<Types>...> MakeTuple(Types&&... Args)
{
	return UE::Core::Private::Tuple::MakeTupleImpl<std::decay_t<Types>...>(Forward<Types>(Args)...);
}


/**
 * Makes a TTuple from some arguments.  Unlike MakeTuple, the TTuple element types are references and retain the
 * same value category of the arguments, like the Forward function.
 *
 * @param  Args  The arguments used to construct the tuple.
 * @return A tuple containing forwarded references to the arguments.
 *
 * Example:
 *
 * template <typename... Ts>
 * void Foo(const TTuple<Ts...>&);
 *
 * void Func(const int32 A, FString&& B)
 * {
 *     // Calls Foo<const int32&, const TCHAR(&)[6], FString&&>(...);
 *     Foo(ForwardAsTuple(A, TEXT("Hello"), MoveTemp(B)));
 * }
 */
template <typename... Types>
UE_FORCEINLINE_HINT TTuple<Types&&...> ForwardAsTuple(Types&&... Args)
{
	return UE::Core::Private::Tuple::MakeTupleImpl<Types&&...>(Forward<Types>(Args)...);
}


/**
 * Creates a new TTuple by applying a functor to each of the elements.
 *
 * @param  Tuple  The tuple to apply the functor to.
 * @param  Func   The functor to apply.
 *
 * @return A new tuple of the transformed elements.
 *
 * Example:
 *
 * float        Overloaded(int32 Arg);
 * char         Overloaded(const TCHAR* Arg);
 * const TCHAR* Overloaded(const FString& Arg);
 *
 * void Func(const TTuple<int32, const TCHAR*, FString>& MyTuple)
 * {
 *     // Equivalent to:
 *     // TTuple<float, char, const TCHAR*> TransformedTuple(Overloaded(MyTuple.Get<0>()), Overloaded(MyTuple.Get<1>()), Overloaded(MyTuple.Get<2>())));
 *     auto TransformedTuple = TransformTuple(MyTuple, [](const auto& Arg) { return Overloaded(Arg); });
 * }
 */
template <typename FuncType, typename... Types>
UE_FORCEINLINE_HINT decltype(auto) TransformTuple(TTuple<Types...>&& Tuple, FuncType Func)
{
	return UE::Core::Private::Tuple::TTransformTuple_Impl<TMakeIntegerSequence<uint32, sizeof...(Types)>>::Do(MoveTemp(Tuple), MoveTemp(Func));
}

template <typename FuncType, typename... Types>
UE_FORCEINLINE_HINT decltype(auto) TransformTuple(const TTuple<Types...>& Tuple, FuncType Func)
{
	return UE::Core::Private::Tuple::TTransformTuple_Impl<TMakeIntegerSequence<uint32, sizeof...(Types)>>::Do(Tuple, MoveTemp(Func));
}


/**
 * Visits each element in the specified tuples in parallel and applies them as arguments to the functor.
 * All specified tuples must have the same number of elements.
 *
 * @param  Func    The functor to apply.
 * @param  Tuples  The tuples whose elements are to be applied to the functor.
 *
 * Example:
 *
 * void Func(const TTuple<int32, const TCHAR*, FString>& Tuple1, const TTuple<bool, float, FName>& Tuple2)
 * {
 *     // Equivalent to:
 *     // Functor(Tuple1.Get<0>(), Tuple2.Get<0>());
 *     // Functor(Tuple1.Get<1>(), Tuple2.Get<1>());
 *     // Functor(Tuple1.Get<2>(), Tuple2.Get<2>());
 *     VisitTupleElements(Functor, Tuple1, Tuple2);
 * }
 */
template <typename FuncType, typename FirstTupleType, typename... TupleTypes>
UE_FORCEINLINE_HINT void VisitTupleElements(FuncType&& Func, FirstTupleType&& FirstTuple, TupleTypes&&... Tuples)
{
	UE::Core::Private::Tuple::TVisitTupleElements_Impl<TMakeIntegerSequence<uint32, TTupleArity<std::decay_t<FirstTupleType>>::Value>>::Do(Forward<FuncType>(Func), Forward<FirstTupleType>(FirstTuple), Forward<TupleTypes>(Tuples)...);
}

/**
 * Tie function for structured unpacking of tuples into individual variables.
 *
 * Example:
 *
 * TTuple<FString, float, TArray<int32>> SomeFunction();
 *
 * FString Ret1;
 * float Ret2;
 * TArray<int32> Ret3;
 *
 * Tie(Ret1, Ret2, Ret3) = SomeFunction();
 *
 * // Now Ret1, Ret2 and Ret3 contain the unpacked return values.
 */
template <typename... Types>
UE_FORCEINLINE_HINT TTuple<Types&...> Tie(Types&... Args)
{
	return TTuple<Types&...>(Args...);
}

///////////////////////////////////////////////////////////////////////////////
// TTuple support for structured bindings - not intended to be used directly //
///////////////////////////////////////////////////////////////////////////////
template <typename... ArgTypes>
struct std::tuple_size<TTuple<ArgTypes...>>
	: std::integral_constant<std::size_t, sizeof...(ArgTypes)>
{
};
template <std::size_t N, typename... ArgTypes>
#if defined(_LIBCPP_VERSION) && _LIBCPP_VERSION < 9000
// libc++ 8.0 and older incorrectly defined std::tuple_element as class, whereas the standard
// says it should be a struct.
class std::tuple_element<N, TTuple<ArgTypes...>>
#else
struct std::tuple_element<N, TTuple<ArgTypes...>>
#endif
{
public:
	using type = typename TTupleElement<N, TTuple<ArgTypes...>>::Type;
};

template <int N, typename TupleType>
	requires (TIsTuple_V<std::decay_t<TupleType>>)
decltype(auto) get(TupleType&& val)
{
	return ((TupleType&&)val).template Get<N>();
}
///////////////////////////////////////
// End of structured binding support //
///////////////////////////////////////

// Overload for std::swap - to support swapping of references
template <typename... Types>
void swap(const TTuple<Types&...>& Lhs, const TTuple<Types&...>& Rhs)
{
	VisitTupleElements(
		[](auto& LhsElement, auto& RhsElement)
		{
			using std::swap;
			swap(LhsElement, RhsElement);
		},
		Lhs,
		Rhs
	);
}
template <typename... Types>
UE_REWRITE void swap(const TTuple<Types&...>& Lhs, TTuple<Types&...>& Rhs)
{
	std::swap(Lhs, AsConst(Rhs));
}
template <typename... Types>
UE_REWRITE void swap(TTuple<Types&...>& Lhs, const TTuple<Types&...>& Rhs)
{
	std::swap(AsConst(Lhs), Rhs);
}
template <typename... Types>
UE_REWRITE void swap(TTuple<Types&...>& Lhs, TTuple<Types&...>& Rhs)
{
	std::swap(AsConst(Lhs), AsConst(Rhs));
}

// Overload for UE Swap - to support swapping of references
template <typename... Types>
void Swap(const TTuple<Types&...>& Lhs, const TTuple<Types&...>& Rhs)
{
	VisitTupleElements(
		[](auto& LhsElement, auto& RhsElement)
		{
			Swap(LhsElement, RhsElement);
		},
		Lhs,
		Rhs
	);
}
template <typename... Types>
UE_REWRITE void Swap(const TTuple<Types&...>& Lhs, TTuple<Types&...>& Rhs)
{
	Swap(Lhs, AsConst(Rhs));
}
template <typename... Types>
UE_REWRITE void Swap(TTuple<Types&...>& Lhs, const TTuple<Types&...>& Rhs)
{
	Swap(AsConst(Lhs), Rhs);
}
template <typename... Types>
UE_REWRITE void Swap(TTuple<Types&...>& Lhs, TTuple<Types&...>& Rhs)
{
	Swap(AsConst(Lhs), AsConst(Rhs));
}

template <uint32 Index>
inline constexpr UE::Core::Private::Tuple::TGetElementByIndex<Index> GetElementByIndex;

template <typename Type>
inline constexpr UE::Core::Private::Tuple::TGetElementByType<Type> GetElementByType;

template <typename... Types>
inline FArchive& operator<<(FArchive& Ar, TTuple<Types...>& Tuple)
{
	TTuple<Types...>::Serialize(Ar, Tuple);
	return Ar;
}

template <typename... Types>
UE_FORCEINLINE_HINT void operator<<(FStructuredArchive::FSlot Slot, TTuple<Types...>& Tuple)
{
	TTuple<Types...>::SerializeStructured(Slot, Tuple);
}
