// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

template <typename... Types>
struct TTuple;

template <typename KeyType, typename ValueType>
using TPair = TTuple<KeyType, ValueType>;
