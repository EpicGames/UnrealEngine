// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreTypes.h"
#include <string.h>
#include <new>
#include <type_traits>

/**
 * Used to declare an untyped array of data with compile-time alignment.
 * It needs to use template specialization as the MS_ALIGN and GCC_ALIGN macros require literal parameters.
 */
template<int32 Size, uint32 Alignment>
struct TAlignedBytes
{
	alignas(Alignment) uint8 Pad[Size];
};

/** An untyped array of data with compile-time alignment and size derived from another type. */
template<typename ElementType>
struct TTypeCompatibleBytes
{
	using ElementTypeAlias_NatVisHelper = ElementType;

	// Trivially constructible and destuctible - users are responsible for managing the lifetime of the inner element.
	TTypeCompatibleBytes() = default;
	~TTypeCompatibleBytes() = default;

	// Noncopyable
	TTypeCompatibleBytes(TTypeCompatibleBytes&&) = delete;
	TTypeCompatibleBytes(const TTypeCompatibleBytes&) = delete;
	TTypeCompatibleBytes& operator=(TTypeCompatibleBytes&&) = delete;
	TTypeCompatibleBytes& operator=(const TTypeCompatibleBytes&) = delete;

	// GetTypedPtr only exists for backwards compatibility - these functions do not exist and cannot be implemented for the reference and void specializations.
	ElementType* GetTypedPtr()
	{
		return (ElementType*)this;
	}
	const ElementType* GetTypedPtr() const
	{
		return (const ElementType*)this;
	}

	using MutableGetType = ElementType&;       // The type returned by Bytes.Get() where Bytes is a non-const lvalue
	using ConstGetType   = const ElementType&; // The type returned by Bytes.Get() where Bytes is a const lvalue
	using RvalueGetType  = ElementType&&;      // The type returned by Bytes.Get() where Bytes is an rvalue (non-const)

	// Gets the inner element - no checks are performed to ensure an element is present.
	ElementType& GetUnchecked() &
	{
		return *(ElementType*)this;
	}
	const ElementType& GetUnchecked() const&
	{
		return *(const ElementType*)this;
	}
	ElementType&& GetUnchecked() &&
	{
		return (ElementType&&)*(ElementType*)this;
	}

	// Emplaces an inner element.
	// Note: no checks are possible to ensure that an element isn't already present.  DestroyUnchecked() must be called to end the element's lifetime.
	template <typename... ArgTypes>
	void EmplaceUnchecked(ArgTypes&&... Args)
	{
		new ((void*)GetTypedPtr()) ElementType((ArgTypes&&)Args...);
	}

	// Destroys the inner element.
	// Note: no checks are possible to ensure that there is an element already present.
	void DestroyUnchecked()
	{
		ElementTypeAlias_NatVisHelper* Ptr = (ElementTypeAlias_NatVisHelper*)this;
		Ptr->ElementTypeAlias_NatVisHelper::~ElementTypeAlias_NatVisHelper();
	}

	alignas(ElementType) uint8 Pad[sizeof(ElementType)];
};

template <typename T>
struct TTypeCompatibleBytes<T&>
{
	using ElementTypeAlias_NatVisHelper = T&;

	// Trivially constructible and destuctible - users are responsible for managing the lifetime of the inner element.
	TTypeCompatibleBytes() = default;
	~TTypeCompatibleBytes() = default;

	// Noncopyable
	TTypeCompatibleBytes(TTypeCompatibleBytes&&) = delete;
	TTypeCompatibleBytes(const TTypeCompatibleBytes&) = delete;
	TTypeCompatibleBytes& operator=(TTypeCompatibleBytes&&) = delete;
	TTypeCompatibleBytes& operator=(const TTypeCompatibleBytes&) = delete;

	using MutableGetType = T&; // The type returned by Bytes.Get() where Bytes is a non-const lvalue
	using ConstGetType   = T&; // The type returned by Bytes.Get() where Bytes is a const lvalue
	using RvalueGetType  = T&; // The type returned by Bytes.Get() where Bytes is an rvalue (non-const)

	// Gets the inner element - no checks are performed to ensure an element is present.
	T& GetUnchecked() const
	{
		return *Ptr;
	}

	// Emplaces an inner element.
	// Note: no checks are possible to ensure that an element isn't already present.  DestroyUnchecked() must be called to end the element's lifetime.
	void EmplaceUnchecked(T& Ref)
	{
		Ptr = &Ref;
	}

	// Destroys the inner element.
	// Note: no checks are possible to ensure that there is an element already present.
	void DestroyUnchecked()
	{
	}

	T* Ptr;
};

template <>
struct TTypeCompatibleBytes<void>
{
	using ElementTypeAlias_NatVisHelper = void;

	// Trivially constructible and destuctible - users are responsible for managing the lifetime of the inner element.
	TTypeCompatibleBytes() = default;
	~TTypeCompatibleBytes() = default;

	// Noncopyable
	TTypeCompatibleBytes(TTypeCompatibleBytes&&) = delete;
	TTypeCompatibleBytes(const TTypeCompatibleBytes&) = delete;
	TTypeCompatibleBytes& operator=(TTypeCompatibleBytes&&) = delete;
	TTypeCompatibleBytes& operator=(const TTypeCompatibleBytes&) = delete;

	using MutableGetType = void; // The type returned by Bytes.Get() where Bytes is a non-const lvalue
	using ConstGetType   = void; // The type returned by Bytes.Get() where Bytes is a const lvalue
	using RvalueGetType  = void; // The type returned by Bytes.Get() where Bytes is an rvalue (non-const)

	// Gets the inner element - no checks are performed to ensure an element is present.
	void GetUnchecked() const
	{
	}

	// Emplaces an inner element.
	// Note: no checks are possible to ensure that an element isn't already present.  DestroyUnchecked() must be called to end the element's lifetime.
	void EmplaceUnchecked()
	{
	}

	// Destroys the inner element.
	// Note: no checks are possible to ensure that there is an element already present.
	void DestroyUnchecked()
	{
	}
};

template <
	typename ToType,
	typename FromType,
	std::enable_if_t<sizeof(ToType) == sizeof(FromType) && std::is_trivially_copyable_v<ToType> && std::is_trivially_copyable_v<FromType>>* = nullptr
>
inline ToType BitCast(const FromType& From)
{
// Ensure we can use this builtin - seems to be present on Clang 9, GCC 11 and MSVC 19.26,
// but gives spurious "non-void function 'BitCast' should return a value" errors on some
// Mac and Android toolchains when building PCHs, so avoid those.
// However, there is a bug in the Clang static analyzer with this builtin: https://github.com/llvm/llvm-project/issues/69922
// Don't use it when performing static analysis until the bug is fixed.
#if !defined(__clang_analyzer__) && PLATFORM_COMPILER_SUPPORTS_BUILTIN_BITCAST // can consider replacing with __has_builtin(__builtin_bit_cast) once there's no special cases
	return __builtin_bit_cast(ToType, From);
#else
	TTypeCompatibleBytes<ToType> Result;
	memcpy(&Result, &From, sizeof(ToType));
	return *Result.GetTypedPtr();
#endif
}
