// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Requires.h"
#include "Misc/Crc.h"

#include <stdint.h>
#include <type_traits>


namespace UE
{
	namespace Private
	{
		[[nodiscard]] inline uint32 MurmurFinalize32(uint32 Hash)
		{
			Hash ^= Hash >> 16;
			Hash *= 0x85ebca6b;
			Hash ^= Hash >> 13;
			Hash *= 0xc2b2ae35;
			Hash ^= Hash >> 16;
			return Hash;
		}

		[[nodiscard]] inline uint64 MurmurFinalize64(uint64 Hash)
		{
			Hash ^= Hash >> 33;
			Hash *= 0xff51afd7ed558ccdull;
			Hash ^= Hash >> 33;
			Hash *= 0xc4ceb9fe1a85ec53ull;
			Hash ^= Hash >> 33;
			return Hash;
		}

		[[nodiscard]] inline uint32 GetTypeHash64(uint64 Value)
		{
			return (uint32)Value + ((uint32)(Value >> 32) * 23);
		}
	}
}

/**
 * Combines two hash values to get a third.
 * Note - this function is not commutative.
 *
 * This function cannot change for backward compatibility reasons.
 * You may want to choose HashCombineFast for a better in-memory hash combining function.
 */
[[nodiscard]] inline constexpr uint32 HashCombine(uint32 A, uint32 C)
{
	uint32 B = 0x9e3779b9;
	A += B;

	A -= B; A -= C; A ^= (C>>13);
	B -= C; B -= A; B ^= (A<<8);
	C -= A; C -= B; C ^= (B>>13);
	A -= B; A -= C; A ^= (C>>12);
	B -= C; B -= A; B ^= (A<<16);
	C -= A; C -= B; C ^= (B>>5);
	A -= B; A -= C; A ^= (C>>3);
	B -= C; B -= A; B ^= (A<<10);
	C -= A; C -= B; C ^= (B>>15);

	return C;
}
template <
	typename... Types
	UE_REQUIRES((std::is_convertible_v<Types, uint32> && ...))
>
[[nodiscard]] inline constexpr uint32 HashCombine(uint32 A, Types... Vals)
{
	if constexpr (sizeof...(Types) > 0)
	{
		A = HashCombine(A, HashCombine((uint32)Vals...));
	}
	return A;
}

/**
 * Combines two hash values to get a third.
 * Note - this function is not commutative.
 *
 * WARNING!  This function is subject to change and should only be used for creating
 *           combined hash values which don't leave the running process,
 *           e.g. GetTypeHash() overloads.
 */
[[nodiscard]] inline constexpr uint32 HashCombineFast(uint32 A, uint32 B)
{
	return A ^ (B + 0x9e3779b9 + (A << 6) + (A >> 2));
}
template <
	typename... Types
	UE_REQUIRES((std::is_convertible_v<Types, uint32> && ...))
>
[[nodiscard]] inline constexpr uint32 HashCombineFast(uint32 A, Types... Vals)
{
	if constexpr (sizeof...(Types) > 0)
	{
		A = HashCombineFast(A, HashCombineFast((uint32)Vals...));
	}
	return A;
}

[[nodiscard]] inline uint32 PointerHash(const void* Key)
{
	const UPTRINT PtrInt = reinterpret_cast<UPTRINT>(Key);
	return UE::Private::GetTypeHash64(UE::Private::MurmurFinalize64(PtrInt));
}

[[nodiscard]] inline uint32 PointerHash(const void* Key, uint32 C)
{
	// we can use HashCombineFast here because pointers are non-persistent
	return HashCombineFast(PointerHash(Key), C);
}


//
// Hash functions for common types.
//
// WARNING!  GetTypeHash result values are not expected to leave the running process.
//           Do not persist them to disk, send them to another running process or
//           expect them to be consistent across multiple runs.
//

template <
	typename ScalarType,
	std::enable_if_t<std::is_scalar_v<ScalarType> && !std::is_same_v<ScalarType, TCHAR*> && !std::is_same_v<ScalarType, const TCHAR*>>* = nullptr
>
[[nodiscard]] inline uint32 GetTypeHash(ScalarType Value)
{
	if constexpr (std::is_integral_v<ScalarType>)
	{
		if constexpr (sizeof(ScalarType) <= 4)
		{
			return Value;
		}
		else if constexpr (sizeof(ScalarType) == 8)
		{
			return UE::Private::GetTypeHash64(Value);
		}
		else if constexpr (sizeof(ScalarType) == 16)
		{
			const uint64 Low = (uint64)Value;
			const uint64 High = (uint64)(Value >> 64);
			return GetTypeHash(Low) ^ GetTypeHash(High);
		}
		else
		{
			static_assert(sizeof(ScalarType) == 0, "Unsupported integral type");
			return 0;
		}
	}
	else if constexpr (std::is_floating_point_v<ScalarType>)
	{
		if constexpr (std::is_same_v<ScalarType, float>)
		{
			return *(uint32*)&Value;
		}
		else if constexpr (std::is_same_v<ScalarType, double>)
		{
			return GetTypeHash(*(uint64*)&Value);
		}
		else
		{
			static_assert(sizeof(ScalarType) == 0, "Unsupported floating point type");
			return 0;
		}
	}
	else if constexpr (std::is_enum_v<ScalarType>)
	{
		return GetTypeHash((__underlying_type(ScalarType))Value);
	}
	else if constexpr (std::is_pointer_v<ScalarType>)
	{
		// Once the TCHAR* deprecations below are removed, we want to prevent accidental string hashing, so this static_assert should be commented back in
		//static_assert(!TIsCharType<std::remove_pointer_t<ScalarType>>::Value, "Pointers to string types should use a PointerHash() or FCrc::Stricmp_DEPRECATED() call depending on requirements");

		return PointerHash(Value);
	}
	else
	{
		static_assert(sizeof(ScalarType) == 0, "Unsupported scalar type");
		return 0;
	}
}

template <
	typename T,
	uint32 N,
	std::enable_if_t<!std::is_same_v<const T, const TCHAR>>* = nullptr
>
UE_DEPRECATED(all, "Hashing arrays is deprecated - use PointerHash() instead to force a conversion to a pointer or GetArrayHash to hash the array contents")
inline uint32 GetTypeHash(T (&Array)[N])
{
	return PointerHash(Array);
}

template <typename T>
[[nodiscard]] inline uint32 GetArrayHash(const T* Ptr, uint64 Size, uint32 PreviousHash = 0)
{
	uint32 Result = PreviousHash;
	while (Size)
	{
		Result = HashCombineFast(Result, GetTypeHash(*Ptr));
		++Ptr;
		--Size;
	}

	return Result;
}

// Use this when inside type that has GetTypeHash() (no in-parameters) implemented. It makes GetTypeHash dispatch in global namespace
template <typename T>
[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetTypeHashHelper(const T& V) { return static_cast<uint32>(GetTypeHash(V)); }

// Use this when in a scope that has AppendHash to dispatch AppendHash using unqualified lookup.
template <typename BuilderType, typename ArgType>
inline void DispatchAppendHash(BuilderType& Builder, const ArgType& Arg)
{
	AppendHash(Builder, Arg);
}
