// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UniquePtr.h"

// This is essentially a reference version of TUniquePtr
// Useful to force heap allocation of a value, e.g. in a TMap to give similar behaviour to TIndirectArray

template <typename T>
class TUniqueObj
{
public:
	using ElementType = T;

	TUniqueObj(const TUniqueObj& other)
		: Obj(MakeUnique<T>(*other))
	{
	}

	// As TUniqueObj's internal pointer can never be null, we can't move that
	// On the other hand we can call the move constructor of "T"
	TUniqueObj(TUniqueObj&& other)
		: Obj(MakeUnique<T>(MoveTemp(*other)))
	{
	}

	template <typename... Args>
	explicit TUniqueObj(Args&&... args)
		: Obj(MakeUnique<T>(Forward<Args>(args)...))
	{
	}

	// Disallow copy-assignment
	TUniqueObj& operator=(const TUniqueObj&) = delete;

	// Move-assignment is implemented as swapping the internal pointer
	TUniqueObj& operator=(TUniqueObj&& other)
	{
		Swap(Obj, other.Obj);
		return *this;
	}

	template <typename Arg>
	TUniqueObj& operator=(Arg&& other)
	{
		*Obj = Forward<Arg>(other);
		return *this;
	}

	      T& Get()       { return *Obj; }
	const T& Get() const { return *Obj; }

	      T* operator->()       { return Obj.Get(); }
	const T* operator->() const { return Obj.Get(); }

	      T& operator*()       { return *Obj; }
	const T& operator*() const { return *Obj; }

	void Serialize(FArchive& Ar)
	{
		Ar << *Obj;
	}

private:
	TUniquePtr<T> Obj;
};

template <typename T>
FArchive& operator<<(FArchive& Ar, TUniqueObj<T>& P)
{
	P.Serialize(Ar);
	return Ar;
}

/**
 * Trait which determines whether or not a type is a TUniqueObj.
 */
template <typename T> constexpr bool TIsTUniqueObj_V                               = false;
template <typename T> constexpr bool TIsTUniqueObj_V<               TUniqueObj<T>> = true;
template <typename T> constexpr bool TIsTUniqueObj_V<const          TUniqueObj<T>> = true;
template <typename T> constexpr bool TIsTUniqueObj_V<      volatile TUniqueObj<T>> = true;
template <typename T> constexpr bool TIsTUniqueObj_V<const volatile TUniqueObj<T>> = true;
