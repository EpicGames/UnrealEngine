// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Concepts/ConvertibleTo.h"
#include "Concepts/PointerConvertibleTo.h"
#include "Concepts/UnboundedArray.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/IsArray.h"
#include "Templates/RemoveExtent.h"
#include "Templates/Requires.h"
#include "Serialization/MemoryLayout.h"
#include <type_traits>

// Single-ownership smart pointer in the vein of std::unique_ptr.
// Use this when you need an object's lifetime to be strictly bound to the lifetime of a single smart pointer.
//
// This class is non-copyable - ownership can only be transferred via a move operation, e.g.:
//
// TUniquePtr<MyClass> Ptr1(new MyClass);    // The MyClass object is owned by Ptr1.
// TUniquePtr<MyClass> Ptr2(Ptr1);           // Error - TUniquePtr is not copyable
// TUniquePtr<MyClass> Ptr3(MoveTemp(Ptr1)); // Ptr3 now owns the MyClass object - Ptr1 is now nullptr.
//
// If you provide a custom deleter, it is up to your deleter to handle null pointers.  This is a departure
// from std::unique_ptr which will not invoke the deleter if the owned pointer is null:
// https://en.cppreference.com/w/cpp/memory/unique_ptr/~unique_ptr

template <typename T>
struct TDefaultDelete
{
	DECLARE_INLINE_TYPE_LAYOUT(TDefaultDelete, NonVirtual);

	TDefaultDelete() = default;
	TDefaultDelete(const TDefaultDelete&) = default;
	TDefaultDelete& operator=(const TDefaultDelete&) = default;
	~TDefaultDelete() = default;

	template <UE::CPointerConvertibleTo<T> U>
	TDefaultDelete(const TDefaultDelete<U>&)
	{
	}

	template <UE::CPointerConvertibleTo<T> U>
	TDefaultDelete& operator=(const TDefaultDelete<U>&)
	{
		return *this;
	}

	void operator()(T* Ptr) const
	{
		// If you get an error here when trying to use a TUniquePtr<FForwardDeclaredType> inside a UObject then:
		//
		// * Declare all your UObject's constructors and destructor in the .h file.
		// * Define all of them in the .cpp file.  You can use UMyObject::UMyObject() = default; to auto-generate
		//   the default constructor and destructor so that they don't have to be manually maintained.
		// * Define a UMyObject(FVTableHelper& Helper) constructor too, otherwise it will be defined in the
		//   .gen.cpp file where your pimpl type doesn't exist.  It cannot be defaulted, but it need not
		//   contain any particular implementation; the object just needs to be garbage collectable.
		//
		// If efficiency is less important than simplicity, you may want to consider
		// using a TPimplPtr instead, though it is also pretty efficient.
		delete Ptr;
	}
};

template <typename T>
struct TDefaultDelete<T[]>
{
	TDefaultDelete() = default;
	TDefaultDelete(const TDefaultDelete&) = default;
	TDefaultDelete& operator=(const TDefaultDelete&) = default;
	~TDefaultDelete() = default;

	template <typename U>
		requires UE::CConvertibleTo<U(*)[], T(*)[]>
	TDefaultDelete(const TDefaultDelete<U[]>&)
	{
	}

	template <typename U>
		requires UE::CConvertibleTo<U(*)[], T(*)[]>
	TDefaultDelete& operator=(const TDefaultDelete<U[]>&)
	{
		return *this;
	}

	template <typename U>
		requires UE::CConvertibleTo<U(*)[], T(*)[]>
	void operator()(U* Ptr) const
	{
		delete [] Ptr;
	}
};

template <typename T, typename Deleter = TDefaultDelete<T>>
class TUniquePtr : private Deleter
{
	DECLARE_INLINE_TYPE_LAYOUT_EXPLICIT_BASES(TUniquePtr, NonVirtual, Deleter);

	template <typename OtherT, typename OtherDeleter>
	friend class TUniquePtr;

public:
	using ElementType = T;

	// Non-copyable
	TUniquePtr(const TUniquePtr&) = delete;
	TUniquePtr& operator=(const TUniquePtr&) = delete;

	/**
	 * Default constructor - initializes the TUniquePtr to null.
	 */
	inline constexpr TUniquePtr()
		: Deleter()
		, Ptr    (nullptr)
	{
	}

	/**
	 * Pointer constructor - takes ownership of the pointed-to object
	 *
	 * @param InPtr The pointed-to object to take ownership of.
	 */
	template <UE::CPointerConvertibleTo<T> U>
	explicit inline TUniquePtr(U* InPtr)
		: Deleter()
		, Ptr    (InPtr)
	{
	}

	/**
	 * Pointer constructor - takes ownership of the pointed-to object
	 *
	 * @param InPtr The pointed-to object to take ownership of.
	 */
	template <UE::CPointerConvertibleTo<T> U>
	explicit inline TUniquePtr(U* InPtr, Deleter&& InDeleter)
		: Deleter(MoveTemp(InDeleter))
		, Ptr    (InPtr)
	{
	}

	/**
	 * Pointer constructor - takes ownership of the pointed-to object
	 *
	 * @param InPtr The pointed-to object to take ownership of.
	 */
	template <UE::CPointerConvertibleTo<T> U>
	explicit inline TUniquePtr(U* InPtr, const Deleter& InDeleter)
		: Deleter(InDeleter)
		, Ptr    (InPtr)
	{
	}

	/**
	 * nullptr constructor - initializes the TUniquePtr to null.
	 */
	inline TUniquePtr(TYPE_OF_NULLPTR)
		: Deleter()
		, Ptr    (nullptr)
	{
	}

	/**
	 * Move constructor
	 */
	inline TUniquePtr(TUniquePtr&& Other)
		: Deleter(MoveTemp(Other.GetDeleter()))
		, Ptr    (Other.Ptr)
	{
		Other.Ptr = nullptr;
	}

	/**
	 * Constructor from rvalues of other (usually derived) types
	 */
	template <UE::CPointerConvertibleTo<T> OtherT, typename OtherDeleter>
		requires (!UE::CArray<OtherT>)
	inline TUniquePtr(TUniquePtr<OtherT, OtherDeleter>&& Other)
		: Deleter(MoveTemp(Other.GetDeleter()))
		, Ptr    (Other.Ptr)
	{
		Other.Ptr = nullptr;
	}

	/**
	 * Move assignment operator
	 */
	inline TUniquePtr& operator=(TUniquePtr&& Other)
	{
		if (this != &Other)
		{
			// We delete last, because we don't want odd side effects if the destructor of T relies on the state of this or Other
			T* OldPtr = Ptr;
			Ptr = Other.Ptr;
			Other.Ptr = nullptr;
			GetDeleter()(OldPtr);
		}

		GetDeleter() = MoveTemp(Other.GetDeleter());

		return *this;
	}

	/**
	 * Assignment operator for rvalues of other (usually derived) types
	 */
	template <UE::CPointerConvertibleTo<T> OtherT, typename OtherDeleter>
		requires (!UE::CArray<OtherT>)
	inline TUniquePtr& operator=(TUniquePtr<OtherT, OtherDeleter>&& Other)
	{
		// We delete last, because we don't want odd side effects if the destructor of T relies on the state of this or Other
		T* OldPtr = Ptr;
		Ptr = Other.Ptr;
		Other.Ptr = nullptr;
		GetDeleter()(OldPtr);

		GetDeleter() = MoveTemp(Other.GetDeleter());

		return *this;
	}

	/**
	 * Nullptr assignment operator
	 */
	inline TUniquePtr& operator=(TYPE_OF_NULLPTR)
	{
		// We delete last, because we don't want odd side effects if the destructor of T relies on the state of this
		T* OldPtr = Ptr;
		Ptr = nullptr;
		GetDeleter()(OldPtr);

		return *this;
	}

	/**
	 * Destructor
	 */
	UE_FORCEINLINE_HINT ~TUniquePtr()
	{
		GetDeleter()(Ptr);
	}

	/**
	 * Tests if the TUniquePtr currently owns an object.
	 *
	 * @return true if the TUniquePtr currently owns an object, false otherwise.
	 */
	[[nodiscard]] bool IsValid() const
	{
		return Ptr != nullptr;
	}

	/**
	 * operator bool
	 *
	 * @return true if the TUniquePtr currently owns an object, false otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
	{
		return IsValid();
	}

	/**
	 * Indirection operator
	 *
	 * @return A pointer to the object owned by the TUniquePtr.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T* operator->() const
	{
		return Ptr;
	}

	/**
	 * Dereference operator
	 *
	 * @return A reference to the object owned by the TUniquePtr.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT std::add_lvalue_reference_t<T> operator*() const
		requires requires { *(T*)nullptr; } // this construct means that operator* is only considered for overload resolution if T is dereferenceable
	{
		return *Ptr;
	}

	/**
	 * Returns a pointer to the owned object without relinquishing ownership.
	 *
	 * @return A copy of the pointer to the object owned by the TUniquePtr, or nullptr if no object is being owned.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T* Get() const
	{
		return Ptr;
	}

	/**
	 * Relinquishes control of the owned object to the caller and nulls the TUniquePtr.
	 *
	 * @return The pointer to the object that was owned by the TUniquePtr, or nullptr if no object was being owned.
	 */
	[[nodiscard]] inline T* Release()
	{
		T* Result = Ptr;
		Ptr = nullptr;
		return Result;
	}

	/**
	 * Gives the TUniquePtr a new object to own, destroying any previously-owned object.
	 *
	 * @param InPtr A pointer to the object to take ownership of.
	 */
	inline void Reset(T* InPtr = nullptr)
	{
		if (Ptr != InPtr)
		{
			// We delete last, because we don't want odd side effects if the destructor of T relies on the state of this
			T* OldPtr = Ptr;
			Ptr = InPtr;
			GetDeleter()(OldPtr);
		}
	}

	/**
	 * Returns a reference to the deleter subobject.
	 *
	 * @return A reference to the deleter.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT Deleter& GetDeleter()
	{
		return static_cast<Deleter&>(*this);
	}

	/**
	 * Returns a reference to the deleter subobject.
	 *
	 * @return A reference to the deleter.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const Deleter& GetDeleter() const
	{
		return static_cast<const Deleter&>(*this);
	}

private:
	using PtrType = T*;
	LAYOUT_FIELD(PtrType, Ptr);

public:
	/**
	 * Equality comparison operator
	 *
	 * @param Lhs The first TUniquePtr to compare.
	 * @param Rhs The second TUniquePtr to compare.
	 *
	 * @return true if the two TUniquePtrs are logically substitutable for each other, false otherwise.
	 */
	template <typename RhsT>
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TUniquePtr<RhsT>& Rhs) const
	{
		return Get() == Rhs.Get();
	}

	/**
	 * Less-than comparison operator
	 *
	 * @param Lhs The first TUniquePtr to compare.
	 * @param Rhs The second TUniquePtr to compare.
	 *
	 * @return true if the inner pointer of the left TUniquePtr is less than the inner pointer of the right, false otherwise.
	 */
	template <typename RhsT>
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator<(const TUniquePtr<RhsT>& Rhs) const
	{
		return Get() < Rhs.Get();
	}

	/**
	 * Greater-than comparison operator
	 *
	 * @param Lhs The first TUniquePtr to compare.
	 * @param Rhs The second TUniquePtr to compare.
	 *
	 * @return true if the inner pointer of the left TUniquePtr is greater than the inner pointer of the right, false otherwise.
	 */
	template <typename RhsT>
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator>(const TUniquePtr<RhsT>& Rhs) const
	{
		return Get() > Rhs.Get();
	}

	/**
	 * Less-than-or-equals comparison operator
	 *
	 * @param Lhs The first TUniquePtr to compare.
	 * @param Rhs The second TUniquePtr to compare.
	 *
	 * @return true if the inner pointer of the left TUniquePtr is less than or equal to the inner pointer of the right, false otherwise.
	 */
	template <typename RhsT>
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator<=(const TUniquePtr<RhsT>& Rhs) const
	{
		return Get() <= Rhs.Get();
	}

	/**
	 * Greater-than-or-equals comparison operator
	 *
	 * @param Lhs The first TUniquePtr to compare.
	 * @param Rhs The second TUniquePtr to compare.
	 *
	 * @return true if the inner pointer of the left TUniquePtr is greater than or equal to the inner pointer of the right, false otherwise.
	 */
	template <typename RhsT>
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator>=(const TUniquePtr<RhsT>& Rhs) const
	{
		return Get() >= Rhs.Get();
	}

	/**
	 * Equality comparison operator against nullptr.
	 *
	 * @param Lhs The TUniquePtr to compare.
	 *
	 * @return true if the TUniquePtr is null, false otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(TYPE_OF_NULLPTR) const
	{
		return !IsValid();
	}

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	/**
	 * Inequality comparison operator
	 *
	 * @param Lhs The first TUniquePtr to compare.
	 * @param Rhs The second TUniquePtr to compare.
	 *
	 * @return false if the two TUniquePtrs are logically substitutable for each other, true otherwise.
	 */
	template <typename RhsT>
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TUniquePtr<RhsT>& Rhs) const
	{
		return Get() != Rhs.Get();
	}

	/**
	 * Inequality comparison operator against nullptr.
	 *
	 * @param Rhs The TUniquePtr to compare.
	 *
	 * @return true if the TUniquePtr is not null, false otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(TYPE_OF_NULLPTR) const
	{
		return IsValid();
	}
#endif

	friend uint32 GetTypeHash(const TUniquePtr& UniquePtr)
	{
		return GetTypeHash(UniquePtr.Get());
	}
};

template <typename T, typename Deleter>
class TUniquePtr<T[], Deleter> : private Deleter
{
	template <typename OtherT, typename OtherDeleter>
	friend class TUniquePtr;

public:
	using ElementType = T;

	// Non-copyable
	TUniquePtr(const TUniquePtr&) = delete;
	TUniquePtr& operator=(const TUniquePtr&) = delete;

	/**
	 * Default constructor - initializes the TUniquePtr to null.
	 */
	inline constexpr TUniquePtr()
		: Deleter()
		, Ptr    (nullptr)
	{
	}

	/**
	 * Pointer constructor - takes ownership of the pointed-to array
	 *
	 * @param InPtr The pointed-to array to take ownership of.
	 */
	template <typename U>
		requires UE::CConvertibleTo<U(*)[], T(*)[]>
	explicit inline TUniquePtr(U* InPtr)
		: Deleter()
		, Ptr    (InPtr)
	{
	}

	/**
	 * Pointer constructor - takes ownership of the pointed-to array
	 *
	 * @param InPtr The pointed-to array to take ownership of.
	 */
	template <typename U>
		requires UE::CConvertibleTo<U(*)[], T(*)[]>
	explicit inline TUniquePtr(U* InPtr, Deleter&& InDeleter)
		: Deleter(MoveTemp(InDeleter))
		, Ptr    (InPtr)
	{
	}

	/**
	 * Pointer constructor - takes ownership of the pointed-to array
	 *
	 * @param InPtr The pointed-to array to take ownership of.
	 */
	template <typename U>
		requires UE::CConvertibleTo<U(*)[], T(*)[]>
	explicit inline TUniquePtr(U* InPtr, const Deleter& InDeleter)
		: Deleter(InDeleter)
		, Ptr    (InPtr)
	{
	}

	/**
	 * nullptr constructor - initializes the TUniquePtr to null.
	 */
	inline constexpr TUniquePtr(TYPE_OF_NULLPTR)
		: Deleter()
		, Ptr    (nullptr)
	{
	}

	/**
	 * Move constructor
	 */
	inline TUniquePtr(TUniquePtr&& Other)
		: Deleter(MoveTemp(Other.GetDeleter()))
		, Ptr    (Other.Ptr)
	{
		Other.Ptr = nullptr;
	}

	/**
	 * Constructor from rvalues of other (usually less qualified) types
	 */
	template <typename OtherT, typename OtherDeleter>
		requires UE::CConvertibleTo<OtherT(*)[], T(*)[]>
	inline TUniquePtr(TUniquePtr<OtherT[], OtherDeleter>&& Other)
		: Deleter(MoveTemp(Other.GetDeleter()))
		, Ptr    (Other.Ptr)
	{
		Other.Ptr = nullptr;
	}

	/**
	 * Move assignment operator
	 */
	inline TUniquePtr& operator=(TUniquePtr&& Other)
	{
		if (this != &Other)
		{
			// We delete last, because we don't want odd side effects if the destructor of T relies on the state of this or Other
			T* OldPtr = Ptr;
			Ptr = Other.Ptr;
			Other.Ptr = nullptr;
			GetDeleter()(OldPtr);
		}

		GetDeleter() = MoveTemp(Other.GetDeleter());

		return *this;
	}

	/**
	 * Assignment operator for rvalues of other (usually less qualified) types
	 */
	template <typename OtherT, typename OtherDeleter>
		requires UE::CConvertibleTo<OtherT(*)[], T(*)[]>
	inline TUniquePtr& operator=(TUniquePtr<OtherT[], OtherDeleter>&& Other)
	{
		// We delete last, because we don't want odd side effects if the destructor of T relies on the state of this or Other
		T* OldPtr = Ptr;
		Ptr = Other.Ptr;
		Other.Ptr = nullptr;
		GetDeleter()(OldPtr);

		GetDeleter() = MoveTemp(Other.GetDeleter());

		return *this;
	}

	/**
	 * Nullptr assignment operator
	 */
	inline TUniquePtr& operator=(TYPE_OF_NULLPTR)
	{
		// We delete last, because we don't want odd side effects if the destructor of T relies on the state of this
		T* OldPtr = Ptr;
		Ptr = nullptr;
		GetDeleter()(OldPtr);

		return *this;
	}

	/**
	 * Destructor
	 */
	UE_FORCEINLINE_HINT ~TUniquePtr()
	{
		GetDeleter()(Ptr);
	}

	/**
	 * Tests if the TUniquePtr currently owns an array.
	 *
	 * @return true if the TUniquePtr currently owns an array, false otherwise.
	 */
	[[nodiscard]] bool IsValid() const
	{
		return Ptr != nullptr;
	}

	/**
	 * operator bool
	 *
	 * @return true if the TUniquePtr currently owns an array, false otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
	{
		return IsValid();
	}

	/**
	 * Indexing operator
	 *
	 * @return A reference to the object at the specified index by the TUniquePtr.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T& operator[](SIZE_T Index) const
	{
		return Ptr[Index];
	}

	/**
	 * Returns a pointer to the owned array without relinquishing ownership.
	 *
	 * @return A copy of the pointer to the array owned by the TUniquePtr, or nullptr if no array is being owned.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T* Get() const
	{
		return Ptr;
	}

	/**
	 * Relinquishes control of the owned array to the caller and nulls the TUniquePtr.
	 *
	 * @return The pointer to the array that was owned by the TUniquePtr, or nullptr if no array was being owned.
	 */
	[[nodiscard]] inline T* Release()
	{
		T* Result = Ptr;
		Ptr = nullptr;
		return Result;
	}

	/**
	 * Gives the TUniquePtr a new array to own, destroying any previously-owned array.
	 *
	 * @param InPtr A pointer to the array to take ownership of.
	 */
	template <typename U>
		requires UE::CConvertibleTo<U(*)[], T(*)[]>
	inline void Reset(U* InPtr)
	{
		// We delete last, because we don't want odd side effects if the destructor of T relies on the state of this
		T* OldPtr = Ptr;
		Ptr = InPtr;
		GetDeleter()(OldPtr);
	}

	inline void Reset(TYPE_OF_NULLPTR InPtr = nullptr)
	{
		// We delete last, because we don't want odd side effects if the destructor of T relies on the state of this
		T* OldPtr = Ptr;
		Ptr = InPtr;
		GetDeleter()(OldPtr);
	}

	/**
	 * Returns a reference to the deleter subobject.
	 *
	 * @return A reference to the deleter.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT Deleter& GetDeleter()
	{
		return static_cast<Deleter&>(*this);
	}

	/**
	 * Returns a reference to the deleter subobject.
	 *
	 * @return A reference to the deleter.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const Deleter& GetDeleter() const
	{
		return static_cast<const Deleter&>(*this);
	}

private:
	T* Ptr;

public:
	/**
	 * Equality comparison operator
	 *
	 * @param Lhs The first TUniquePtr to compare.
	 * @param Rhs The second TUniquePtr to compare.
	 *
	 * @return true if the two TUniquePtrs are logically substitutable for each other, false otherwise.
	 */
	template <typename RhsT>
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TUniquePtr<RhsT>& Rhs) const
	{
		return Get() == Rhs.Get();
	}

	/**
	 * Less-than comparison operator
	 *
	 * @param Lhs The first TUniquePtr to compare.
	 * @param Rhs The second TUniquePtr to compare.
	 *
	 * @return true if the inner pointer of the left TUniquePtr is less than the inner pointer of the right, false otherwise.
	 */
	template <typename RhsT>
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator<(const TUniquePtr<RhsT>& Rhs) const
	{
		return Get() < Rhs.Get();
	}

	/**
	 * Greater-than comparison operator
	 *
	 * @param Lhs The first TUniquePtr to compare.
	 * @param Rhs The second TUniquePtr to compare.
	 *
	 * @return true if the inner pointer of the left TUniquePtr is greater than the inner pointer of the right, false otherwise.
	 */
	template <typename RhsT>
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator>(const TUniquePtr<RhsT>& Rhs) const
	{
		return Get() > Rhs.Get();
	}

	/**
	 * Less-than-or-equals comparison operator
	 *
	 * @param Lhs The first TUniquePtr to compare.
	 * @param Rhs The second TUniquePtr to compare.
	 *
	 * @return true if the inner pointer of the left TUniquePtr is less than or equal to the inner pointer of the right, false otherwise.
	 */
	template <typename RhsT>
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator<=(const TUniquePtr<RhsT>& Rhs) const
	{
		return Get() <= Rhs.Get();
	}

	/**
	 * Greater-than-or-equals comparison operator
	 *
	 * @param Lhs The first TUniquePtr to compare.
	 * @param Rhs The second TUniquePtr to compare.
	 *
	 * @return true if the inner pointer of the left TUniquePtr is greater than or equal to the inner pointer of the right, false otherwise.
	 */
	template <typename RhsT>
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator>=(const TUniquePtr<RhsT>& Rhs) const
	{
		return Get() >= Rhs.Get();
	}

	/**
	 * Equality comparison operator against nullptr.
	 *
	 * @param Lhs The TUniquePtr to compare.
	 *
	 * @return true if the TUniquePtr is null, false otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(TYPE_OF_NULLPTR) const
	{
		return !IsValid();
	}

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	/**
	 * Inequality comparison operator
	 *
	 * @param Lhs The first TUniquePtr to compare.
	 * @param Rhs The second TUniquePtr to compare.
	 *
	 * @return false if the two TUniquePtrs are logically substitutable for each other, true otherwise.
	 */
	template <typename RhsT>
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TUniquePtr<RhsT>& Rhs) const
	{
		return Get() != Rhs.Get();
	}

	/**
	 * Inequality comparison operator against nullptr.
	 *
	 * @param Rhs The TUniquePtr to compare.
	 *
	 * @return true if the TUniquePtr is not null, false otherwise.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(TYPE_OF_NULLPTR) const
	{
		return IsValid();
	}
#endif

	[[nodiscard]] friend uint32 GetTypeHash(const TUniquePtr& UniquePtr)
	{
		return GetTypeHash(UniquePtr.Get());
	}
};

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
template <typename T>
[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(TYPE_OF_NULLPTR, const TUniquePtr<T>& Rhs)
{
	return !Rhs.IsValid();
}

template <typename T>
[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(TYPE_OF_NULLPTR, const TUniquePtr<T>& Rhs)
{
	return Rhs.IsValid();
}
#endif

// Trait which allows TUniquePtr to be default constructed by memsetting to zero.
template <typename T>
struct TIsZeroConstructType<TUniquePtr<T>>
{
	enum { Value = true };
};

// Trait which allows TUniquePtr to be memcpy'able from pointers.
template <typename T>
struct TIsBitwiseConstructible<TUniquePtr<T>, T*>
{
	enum { Value = true };
};

/**
 * Allocates a new object of type T with the given arguments and returns it as a TUniquePtr.  Disabled for array-type TUniquePtrs.
 * The object is value-initialized, which will call a user-defined default constructor if it exists, but a trivial type will be zeroed.
 *
 * @param Args The arguments to pass to the constructor of T.
 *
 * @return A TUniquePtr which points to a newly-constructed T with the specified Args.
 */
template <typename T, typename... ArgTypes>
	requires (!UE::CArray<T>)
[[nodiscard]] UE_FORCEINLINE_HINT TUniquePtr<T> MakeUnique(ArgTypes&&... Args)
{
	return TUniquePtr<T>(new T(Forward<ArgTypes>(Args)...));
}

/**
 * Allocates a new object of type T with the given arguments and returns it as a TUniquePtr.  Disabled for array-type TUniquePtrs.
 * The object is default-initialized, which will call a user-defined default constructor if it exists, but a trivial type will be uninitialized.
 *
 * @param Args The arguments to pass to the constructor of T.
 *
 * @return A TUniquePtr which points to a newly-constructed T with the specified Args.
 */
template <typename T>
	requires (!UE::CArray<T>)
[[nodiscard]] UE_FORCEINLINE_HINT TUniquePtr<T> MakeUniqueForOverwrite()
{
	return TUniquePtr<T>(new T);
}

/**
 * Allocates a new array of type T with the given size and returns it as a TUniquePtr.  Only enabled for array-type TUniquePtrs.
 * Elements are value-initialized, which will call a user-defined default constructor if it exists, but causes trivial types to be zeroed.
 *
 * @param Size The size of the array to allocate.
 *
 * @return A TUniquePtr which points to a newly-constructed T array of the specified Size.
 */
template <UE::CUnboundedArray T>
[[nodiscard]] inline TUniquePtr<T> MakeUnique(SIZE_T Size)
{
	using ElementType = std::remove_extent_t<T>;
	return TUniquePtr<T>(new ElementType[Size]());
}

/**
 * Allocates a new array of type T with the given size and returns it as a TUniquePtr.  Only enabled for array-type TUniquePtrs.
 * Elements are default-initialized, which will call a user-defined default constructor if it exists, but causes trivial types to be left uninitialized.
 *
 * @param Size The size of the array to allocate.
 *
 * @return A TUniquePtr which points to a newly-constructed T array of the specified Size.
 */
template <UE::CUnboundedArray T>
[[nodiscard]] inline TUniquePtr<T> MakeUniqueForOverwrite(SIZE_T Size)
{
	using ElementType = std::remove_extent_t<T>;
	return TUniquePtr<T>(new ElementType[Size]);
}

/**
 * Trait which determines whether or not a type is a TUniquePtr.
 */
template <typename T>                   constexpr bool TIsTUniquePtr_V                                        = false;
template <typename T, typename Deleter> constexpr bool TIsTUniquePtr_V<               TUniquePtr<T, Deleter>> = true;
template <typename T, typename Deleter> constexpr bool TIsTUniquePtr_V<const          TUniquePtr<T, Deleter>> = true;
template <typename T, typename Deleter> constexpr bool TIsTUniquePtr_V<      volatile TUniquePtr<T, Deleter>> = true;
template <typename T, typename Deleter> constexpr bool TIsTUniquePtr_V<const volatile TUniquePtr<T, Deleter>> = true;
