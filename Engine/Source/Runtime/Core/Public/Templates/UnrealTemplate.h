// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/IsPointer.h"
#include "HAL/UnrealMemory.h"
#include "Templates/CopyQualifiersAndRefsFromTo.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/RemoveReference.h"
#include "Templates/Requires.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/Identity.h"
#include "Traits/IsContiguousContainer.h"
#include "Traits/UseBitwiseSwap.h"
#include <type_traits>

/*-----------------------------------------------------------------------------
	Standard templates.
-----------------------------------------------------------------------------*/

/** This is used to provide type specific behavior for a copy which cannot change the value of B. */
template<typename T>
inline void Move(T& A,typename TMoveSupportTraits<T>::Copy B)
{
	// Destruct the previous value of A.
	A.~T();

	// Use placement new and a copy constructor so types with const members will work.
	::new((void*)&A) T(B);
}

/** This is used to provide type specific behavior for a move which may change the value of B. */
template<typename T>
inline void Move(T& A,typename TMoveSupportTraits<T>::Move B)
{
	// Destruct the previous value of A.
	A.~T();

	// Use placement new and a copy constructor so types with const members will work.
	::new((void*)&A) T(MoveTemp(B));
}

namespace UE::Core::Private
{
	template <typename T>
	[[nodiscard]] constexpr auto GetDataImpl(T&& Container) -> decltype(Container.GetData())
	{
		return Container.GetData();
	}

	template <
		typename T
		UE_REQUIRES(std::is_pointer_v<decltype(+std::declval<T>())>)
	>
	[[nodiscard]] constexpr auto GetDataImpl(T&& Container)
	{
		return +Container;
	}

	template <typename T>
	[[nodiscard]] constexpr auto GetNumImpl(const T& Container) -> decltype(Container.Num())
	{
		return Container.Num();
	}

	template <
		typename T
		UE_REQUIRES(std::is_pointer_v<decltype(+std::declval<T>())>)
	>
	[[nodiscard]] constexpr auto GetNumImpl(const T& Container) -> SIZE_T
	{
		return sizeof(Container) / sizeof(*Container);
	}
}

/**
 * Generically gets the data pointer of a contiguous container
 */
template <
	typename T
	UE_REQUIRES(TIsContiguousContainer<T>::Value)
>
[[nodiscard]] constexpr auto GetData(T&& Container) -> decltype(UE::Core::Private::GetDataImpl((T&&)Container))
{
	return UE::Core::Private::GetDataImpl((T&&)Container);
}

template <typename T>
constexpr const T* GetData(std::initializer_list<T> List)
{
	return List.begin();
}

/**
 * Forwards to an unqualified GetData(), but can be called from within a scope
 * where GetData() is a member and hides any other
 */
template <typename T>
[[nodiscard]] UE_REWRITE constexpr auto GetDataHelper(T&& Arg) -> decltype(GetData((T&&)Arg))
{
	return GetData((T&&)Arg);
}

/**
* Generically gets the number of items in a contiguous container
*/
template <
	typename T
	UE_REQUIRES(TIsContiguousContainer<T>::Value)
>
[[nodiscard]] constexpr auto GetNum(const T& Container) -> decltype(UE::Core::Private::GetNumImpl(Container))
{
	return UE::Core::Private::GetNumImpl(Container);
}

/**
 * Gets the number of items in an initializer list.
 *
 * The return type is int32 for compatibility with other code in the engine.
 * Realistically, an initializer list should not exceed the limits of int32.
 */
template <typename T>
constexpr int32 GetNum(std::initializer_list<T> List)
{
	return static_cast<int32>(List.size());
}

/**
 * Returns a non-const reference type as const.
 */
template <typename T>
constexpr UE_FORCEINLINE_HINT const T& AsConst(T& Ref)
{
	return Ref;
}

/**
 * Disallowed for rvalue references because it cannot extend their lifetime.
 */
template <typename T>
void AsConst(const T&& Ref) = delete;

/**
 * Returns a non-const reference type as const.
 * This overload is only required until the pointer overloads are removed.
 */
template <typename T, SIZE_T N>
constexpr UE_FORCEINLINE_HINT const T (&AsConst(T (&Array)[N]))[N]
{
	return Array;
}

/** Test if value can make a lossless static_cast roundtrip via OutType without a sign change */
template<typename OutType, typename InType>
constexpr bool IntFitsIn(InType In)
{
	static_assert(std::is_integral_v<InType> && std::is_integral_v<OutType>, "Only integers supported");
	
	OutType Out = static_cast<OutType>(In);
	bool bRoundtrips = In == static_cast<InType>(Out);
	
	// Signed <-> unsigned cast requires sign test, signed -> smaller signed is covered by roundtrip sign-extension.
	if constexpr ((static_cast<InType>(-1) < InType{}) != (static_cast<OutType>(-1) < OutType{}))
	{
		return bRoundtrips && (In < InType{} == Out < OutType{});
	}
	else
	{
		return bRoundtrips;
	}
}

/** Cast and check that value fits in OutType */
template<typename OutType, typename InType>
OutType IntCastChecked(InType In)
{
	if constexpr (std::is_signed_v<InType>)
	{
		checkf(IntFitsIn<OutType>(In), TEXT("Loss of data caused by narrowing conversion, In = %" INT64_FMT), (int64)In);
	}
	else
	{
		checkf(IntFitsIn<OutType>(In), TEXT("Loss of data caused by narrowing conversion, In = %" UINT64_FMT), (uint64)In);
	}
	return static_cast<OutType>(In);
}

/** Test if value can make a static_cast roundtrip via OutType whilst maintaining precision */
template<typename OutType, typename InType>
constexpr bool FloatFitsIn(InType In, InType Precision)
{
	static_assert(std::is_floating_point_v<InType> && std::is_floating_point_v<OutType>, "Only floating point supported");
	
	OutType Out = static_cast<OutType>(In);
	return fabs(static_cast<InType>(Out) - In) <= Precision;
}

template<typename OutType, typename InType>
OutType FloatCastChecked(InType In, InType Precision)
{
	checkf(FloatFitsIn<OutType>(In, Precision), TEXT("Loss of data caused by narrowing conversion"));
	return static_cast<OutType>(In);
}

/*----------------------------------------------------------------------------
	Standard macros.
----------------------------------------------------------------------------*/

#ifdef __clang__
	template <
		typename T
		UE_REQUIRES(__is_array(T))
	>
	auto UEArrayCountHelper(T& t) -> char(&)[sizeof(t) / sizeof(t[0]) + 1];
#else
	template <typename T, uint32 N>
	char (&UEArrayCountHelper(const T (&)[N]))[N + 1];
#endif

// Number of elements in an array.
#define UE_ARRAY_COUNT( array ) (sizeof(UEArrayCountHelper(array)) - 1)

// Offset of a struct member.
#ifdef __clang__
#define STRUCT_OFFSET( struc, member )	__builtin_offsetof(struc, member)
#else
#define STRUCT_OFFSET( struc, member )	offsetof(struc, member)
#endif

#if PLATFORM_VTABLE_AT_END_OF_CLASS
	#error need implementation
#else
	#define VTABLE_OFFSET( Class, MultipleInheritenceParent )	( ((PTRINT) static_cast<MultipleInheritenceParent*>((Class*)1)) - 1)
#endif

namespace UE::Core::Private
{
	template <typename T, T Val>
	constexpr T TForceConstEval_V = Val;
}

// Forces an expression to be evaluated at compile-time, even if it is part of a runtime expression:
//
// Example:
//   // Arg 3 is evaluated at runtime as it's used in a runtime context, despite the function being marked constexpr and having a compile-time argument.
//   // Requires an optimizer pass to eliminate.
//   RegisterTypeWithSizeAndLog2Alignment("MyType", sizeof(FMyType), FMath::ConstExprCeilLogTwo(alignof(FMyType)));
//
//   // Arg 3 is evaluated at compile-time, but is non-intuitive and requires another variable to be introduced
//   constexpr SIZE_T AlignOfMyTypeLog2 = alignof(FMyType);
//   RegisterTypeWithSizeAndLog2Alignment("MyType", sizeof(FMyType), AlignOfMyTypeLog2);
//
//   // Arg 3 is evaluated at compile time with a more direct syntax
//   RegisterTypeWithSizeAndLog2Alignment("MyType", sizeof(FMyType), UE_FORCE_CONSTEVAL(FMath::ConstExprCeilLogTwo(alignof(FMyType))));
#define UE_FORCE_CONSTEVAL(expr) UE::Core::Private::TForceConstEval_V<std::decay_t<decltype(expr)>, (expr)>

/**
 * works just like std::min_element.
 */
template<class ForwardIt> inline
ForwardIt MinElement(ForwardIt First, ForwardIt Last)
{
	ForwardIt Result = First;
	for (; ++First != Last; )
	{
		if (*First < *Result) 
		{
			Result = First;
		}
	}
	return Result;
}

/**
 * works just like std::min_element.
 */
template<class ForwardIt, class PredicateType> inline
ForwardIt MinElement(ForwardIt First, ForwardIt Last, PredicateType Predicate)
{
	ForwardIt Result = First;
	for (; ++First != Last; )
	{
		if (Predicate(*First,*Result))
		{
			Result = First;
		}
	}
	return Result;
}

/**
* works just like std::max_element.
*/
template<class ForwardIt> inline
ForwardIt MaxElement(ForwardIt First, ForwardIt Last)
{
	ForwardIt Result = First;
	for (; ++First != Last; )
	{
		if (*Result < *First) 
		{
			Result = First;
		}
	}
	return Result;
}

/**
* works just like std::max_element.
*/
template<class ForwardIt, class PredicateType> inline
ForwardIt MaxElement(ForwardIt First, ForwardIt Last, PredicateType Predicate)
{
	ForwardIt Result = First;
	for (; ++First != Last; )
	{
		if (Predicate(*Result,*First))
		{
			Result = First;
		}
	}
	return Result;
}

/**
 * utility template for a class that should not be copyable.
 * Derive from this class to make your class non-copyable
 */
class FNoncopyable
{
protected:
	// ensure the class cannot be constructed directly
	FNoncopyable() {}
	// the class should not be used polymorphically
	~FNoncopyable() = default;
private:
	FNoncopyable(const FNoncopyable&);
	FNoncopyable& operator=(const FNoncopyable&);
};

/** 
 * exception-safe guard around saving/restoring a value.
 * Commonly used to make sure a value is restored 
 * even if the code early outs in the future.
 * Usage:
 *  	TGuardValue<bool> GuardSomeBool(bSomeBool, false); // Sets bSomeBool to false, and restores it in dtor.
 */
template <typename RefType, typename AssignedType = RefType>
struct TGuardValue : private FNoncopyable
{
	[[nodiscard]] TGuardValue(RefType& ReferenceValue, const AssignedType& NewValue)
	: RefValue(ReferenceValue), OriginalValue(ReferenceValue)
	{
		RefValue = NewValue;
	}
	~TGuardValue()
	{
		RefValue = OriginalValue;
	}

	/**
	 * Provides read-only access to the original value of the data being tracked by this struct
	 *
	 * @return	a const reference to the original data value
	 */
	UE_FORCEINLINE_HINT const AssignedType& GetOriginalValue() const
	{
		return OriginalValue;
	}

private:
	RefType& RefValue;
	AssignedType OriginalValue;
};


/**
 * exception-safe guard around saving/restoring a value.
 * Commonly used to make sure a value is restored
 * even if the code early outs in the future.
 * Usage:
 *  	TOptionalGuardValue<bool> GuardSomeBool(bSomeBool, false); // Sets bSomeBool to false, and restores it in dtor.
 */
template <typename RefType, typename AssignedType = RefType>
struct TOptionalGuardValue : private FNoncopyable
{
	[[nodiscard]] TOptionalGuardValue(RefType& ReferenceValue, const AssignedType& NewValue)
		: RefValue(ReferenceValue), OriginalValue(ReferenceValue)
	{
		if (RefValue != NewValue)
		{
			RefValue = NewValue;
		}
	}
	~TOptionalGuardValue()
	{
		if (RefValue != OriginalValue)
		{
			RefValue = OriginalValue;
		}
	}

	/**
	 * Provides read-only access to the original value of the data being tracked by this struct
	 *
	 * @return	a const reference to the original data value
	 */
	UE_FORCEINLINE_HINT const AssignedType& GetOriginalValue() const
	{
		return OriginalValue;
	}

private:
	RefType& RefValue;
	AssignedType OriginalValue;
};

template <typename FuncType>
struct TGuardValue_Bitfield_Cleanup : public FNoncopyable
{
	[[nodiscard]] explicit TGuardValue_Bitfield_Cleanup(FuncType&& InFunc)
		: Func(MoveTemp(InFunc))
	{
	}

	~TGuardValue_Bitfield_Cleanup()
	{
		Func();
	}

private:
	FuncType Func;
};

/** 
 * Macro variant on TGuardValue<bool> that can deal with bitfields which cannot be passed by reference in to TGuardValue
 */
#define FGuardValue_Bitfield(ReferenceValue, NewValue) \
	const bool UE_JOIN(TempBitfield, __LINE__) = ReferenceValue; \
	ReferenceValue = NewValue; \
	const TGuardValue_Bitfield_Cleanup<TFunction<void()>> UE_JOIN(TempBitfieldCleanup, __LINE__)([&](){ ReferenceValue = UE_JOIN(TempBitfield, __LINE__); });

/** 
 * Commonly used to make sure a value is incremented, and then decremented anyway the function can terminate.
 * Usage:
 *  	TScopeCounter<int32> BeginProcessing(ProcessingCount); // increments ProcessingCount, and decrements it in the dtor
 */
template <typename Type>
struct TScopeCounter : private FNoncopyable
{
	[[nodiscard]] explicit TScopeCounter(Type& ReferenceValue)
		: RefValue(ReferenceValue)
	{
		++RefValue;
	}
	~TScopeCounter()
	{
		--RefValue;
	}

private:
	Type& RefValue;
};


/**
 * Helper class to make it easy to use key/value pairs with a container.
 */
template <typename KeyType, typename ValueType>
struct TKeyValuePair
{
	TKeyValuePair( const KeyType& InKey, const ValueType& InValue )
	:	Key(InKey), Value(InValue)
	{
	}
	TKeyValuePair( const KeyType& InKey )
	:	Key(InKey)
	{
	}
	TKeyValuePair()
	{
	}
	bool operator==( const TKeyValuePair& Other ) const
	{
		return Key == Other.Key;
	}
	bool operator!=( const TKeyValuePair& Other ) const
	{
		return Key != Other.Key;
	}
	bool operator<( const TKeyValuePair& Other ) const
	{
		return Key < Other.Key;
	}
	UE_FORCEINLINE_HINT bool operator()( const TKeyValuePair& A, const TKeyValuePair& B ) const
	{
		return A.Key < B.Key;
	}
	KeyType		Key;
	ValueType	Value;
};

//
// Macros that can be used to specify multiple template parameters in a macro parameter.
// This is necessary to prevent the macro parsing from interpreting the template parameter
// delimiting comma as a macro parameter delimiter.
// 

#define TEMPLATE_PARAMETERS2(X,Y) X,Y


/**
 * Removes one level of pointer from a type, e.g.:
 *
 * TRemovePointer<      int32  >::Type == int32
 * TRemovePointer<      int32* >::Type == int32
 * TRemovePointer<      int32**>::Type == int32*
 * TRemovePointer<const int32* >::Type == const int32
 */
template <typename T> struct TRemovePointer     { typedef T Type; };
template <typename T> struct TRemovePointer<T*> { typedef T Type; };

/**
 * MoveTemp will cast a reference to an rvalue reference.
 * This is UE's equivalent of std::move except that it will not compile when passed an rvalue or
 * const object, because we would prefer to be informed when MoveTemp will have no effect.
 */
template <typename T>
UE_INTRINSIC_CAST UE_REWRITE constexpr std::remove_reference_t<T>&& MoveTemp(T&& Obj) noexcept
{
	using CastType = std::remove_reference_t<T>;

	// Validate that we're not being passed an rvalue or a const object - the former is redundant, the latter is almost certainly a mistake
	static_assert(std::is_lvalue_reference_v<T>, "MoveTemp called on an rvalue");
	static_assert(!std::is_same_v<CastType&, const CastType&>, "MoveTemp called on a const object");

	return (CastType&&)Obj;
}

/**
 * MoveTempIfPossible will cast a reference to an rvalue reference.
 * This is UE's equivalent of std::move.  It doesn't static assert like MoveTemp, because it is useful in
 * templates or macros where it's not obvious what the argument is, but you want to take advantage of move semantics
 * where you can but not stop compilation.
 */
template <typename T>
UE_INTRINSIC_CAST UE_REWRITE constexpr std::remove_reference_t<T>&& MoveTempIfPossible(T&& Obj) noexcept
{
	using CastType = std::remove_reference_t<T>;
	return (CastType&&)Obj;
}

/**
 * CopyTemp will enforce the creation of a prvalue which can bind to rvalue reference parameters.
 * Unlike MoveTemp, a source lvalue will never be modified. (i.e. a copy will always be made)
 * There is no std:: equivalent, though there is the exposition function std::decay-copy:
 * https://eel.is/c++draft/expos.only.func
 * CopyTemp(<rvalue>) is regarded as an error and will not compile, similarly to how MoveTemp(<rvalue>)
 * does not compile, and CopyTempIfNecessary should be used instead when the nature of the
 * argument is not known in advance.
 */
template <typename T>
UE_REWRITE T CopyTemp(T& Val)
{
	return const_cast<const T&>(Val);
}

template <typename T>
UE_REWRITE T CopyTemp(const T& Val)
{
	return Val;
}

/**
 * CopyTempIfNecessary will enforce the creation of a prvalue.
 * This is UE's equivalent of the exposition std::decay-copy:
 * https://eel.is/c++draft/expos.only.func
 * It doesn't static assert like CopyTemp, because it is useful in
 * templates or macros where it's not obvious what the argument is, but you want to
 * create a PR value without stopping compilation.
 */
template <typename T>
UE_REWRITE constexpr std::decay_t<T> CopyTempIfNecessary(T&& Val)
{
	return (T&&)Val;
}

/**
 * Forward will cast a reference to an rvalue reference.
 * This is UE's equivalent of std::forward.
 */
template <typename T>
UE_INTRINSIC_CAST UE_REWRITE constexpr T&& Forward(std::remove_reference_t<T>& Obj) noexcept
{
	return (T&&)Obj;
}

template <typename T>
UE_INTRINSIC_CAST UE_REWRITE constexpr T&& Forward(std::remove_reference_t<T>&& Obj) noexcept
{
	return (T&&)Obj;
}

/**
 * Swap two values.  Assumes the types are trivially relocatable.
 */
template <typename T>
constexpr inline void Swap(T& A, T& B)
{
	// std::is_swappable isn't correct here, because we allow bitwise swapping of types containing e.g. const and reference members,
	// but we don't want to allow swapping of types which are UE_NONCOPYABLE or equivalent.  We also allow bitwise swapping of arrays, so
	// extents should be removed first.
	static_assert(std::is_move_constructible_v<std::remove_all_extents_t<T>>, "Cannot swap non-movable types");

	if constexpr (TUseBitwiseSwap<T>::Value)
	{
		struct FAlignedBytes
		{
			alignas(T) char Bytes[sizeof(T)];
		};

		FAlignedBytes Temp;
		*(FAlignedBytes*)&Temp = *(FAlignedBytes*)&A;
		*(FAlignedBytes*)&A    = *(FAlignedBytes*)&B;
		*(FAlignedBytes*)&B    = *(FAlignedBytes*)&Temp;
	}
	else
	{
		T Temp = MoveTemp(A);
		A = MoveTemp(B);
		B = MoveTemp(Temp);
	}
}

template <typename T>
UE_REWRITE constexpr void Exchange(T& A, T& B)
{
	Swap(A, B);
}

/**
 * This exists to avoid a Visual Studio bug where using a cast to forward an rvalue reference array argument
 * to a pointer parameter will cause bad code generation.  Wrapping the cast in a function causes the correct
 * code to be generated.
 */
template <typename T, typename ArgType>
UE_REWRITE T StaticCast(ArgType&& Arg)
{
	return static_cast<T>(Arg);
}

/**
 * TRValueToLValueReference converts any rvalue reference type into the equivalent lvalue reference, otherwise returns the same type.
 */
template <typename T> struct TRValueToLValueReference      { typedef T  Type; };
template <typename T> struct TRValueToLValueReference<T&&> { typedef T& Type; };

/**
 * Reverses the order of the bits of a value.
 * This is a constrained template to ensure that no undesirable conversions occur.  Overloads for other types can be added in the same way.
 *
 * @param Bits - The value to bit-swap.
 * @return The bit-swapped value.
 */
template <
	typename T
	UE_REQUIRES(std::is_same_v<T, uint32>)
>
inline T ReverseBits( T Bits )
{
	Bits = ( Bits << 16) | ( Bits >> 16);
	Bits = ( (Bits & 0x00ff00ff) << 8 ) | ( (Bits & 0xff00ff00) >> 8 );
	Bits = ( (Bits & 0x0f0f0f0f) << 4 ) | ( (Bits & 0xf0f0f0f0) >> 4 );
	Bits = ( (Bits & 0x33333333) << 2 ) | ( (Bits & 0xcccccccc) >> 2 );
	Bits = ( (Bits & 0x55555555) << 1 ) | ( (Bits & 0xaaaaaaaa) >> 1 );
	return Bits;
}

/**
 * Generates a bitmask with a given number of bits set.
 */
template <typename T>
UE_FORCEINLINE_HINT T BitMask( uint32 Count );

template <>
inline uint64 BitMask<uint64>( uint32 Count )
{
	checkSlow(Count <= 64);
	return (uint64(Count < 64) << Count) - 1;
}

template <>
inline uint32 BitMask<uint32>( uint32 Count )
{
	checkSlow(Count <= 32);
	return uint32(uint64(1) << Count) - 1;
}

template <>
inline uint16 BitMask<uint16>( uint32 Count )
{
	checkSlow(Count <= 16);
	return uint16((uint32(1) << Count) - 1);
}

template <>
inline uint8 BitMask<uint8>( uint32 Count )
{
	checkSlow(Count <= 8);
	return uint8((uint32(1) << Count) - 1);
}


/** Template for initializing a singleton at the boot. */
template< class T >
struct TForceInitAtBoot
{
	TForceInitAtBoot()
	{
		T::Get();
	}
};

/** Used to avoid cluttering code with ifdefs. */
struct FNoopStruct
{
	FNoopStruct()
	{}

	~FNoopStruct() = default;
};

/**
 * Equivalent to std::declval.
 *
 * Note that this function is unimplemented, and is only intended to be used in unevaluated contexts, like sizeof and trait expressions.
 */
template <typename T>
T&& DeclVal();

/**
 * Uses implicit conversion to create an instance of a specific type.
 * Useful to make things clearer or circumvent unintended type deduction in templates.
 * Safer than C casts and static_casts, e.g. does not allow down-casts
 *
 * @param Obj  The object (usually pointer or reference) to convert.
 *
 * @return The object converted to the specified type.
 */
template <typename T>
UE_REWRITE constexpr T ImplicitConv(typename TIdentity<T>::Type Obj)
{
	return Obj;
}

/**
 * ForwardAsBase will cast a reference to an rvalue reference of a base type.
 * This allows the perfect forwarding of a reference as a base class.
 */
template <
	typename T,
	typename Base
	UE_REQUIRES(std::is_convertible_v<std::remove_reference_t<T>*, const volatile Base*>)
>
UE_INTRINSIC_CAST UE_REWRITE decltype(auto) ForwardAsBase(std::remove_reference_t<T>& Obj)
{
	return (TCopyQualifiersAndRefsFromTo_T<T&&, Base>)Obj;
}

template <
	typename T,
	typename Base
	UE_REQUIRES(std::is_convertible_v<std::remove_reference_t<T>*, const volatile Base*>)
>
UE_INTRINSIC_CAST UE_REWRITE decltype(auto) ForwardAsBase(std::remove_reference_t<T>&& Obj)
{
	return (TCopyQualifiersAndRefsFromTo_T<T&&, Base>)Obj;
}

/**
 * ForwardLike will cast a reference to an object with the cv qualifiers and value category
 * of another type.  This allows perfect forwarding of subobjects with the same cvrefs as
 * the outer type.  Qualifiers and refs from each type are composed, so const will never
 * be cast away, nor will an lvalue be cast to an rvalue.
 *
 * Example:
 *
 * template <typename T>
 * void GetInnerA(T&& Outer)
 * {
 *     // Has type const TypeOfA& if Outer is a const lvalue.
 *     // Has type TypeOfA&& if Outer is an rvalue.
 *     // etc.
 *     auto&& OuterA = ForwardLike<T>(Outer.A);
 * }
 */
template <typename T, typename U>
UE_INTRINSIC_CAST UE_REWRITE decltype(auto) ForwardLike(U&& Obj)
{
	return (TCopyQualifiersAndRefsFromTo_T<T&&, std::remove_reference_t<U>>)Obj;
}
