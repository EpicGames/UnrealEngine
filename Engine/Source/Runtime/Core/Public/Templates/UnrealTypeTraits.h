// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnrealTypeTraits.h: Unreal type traits definitions.
	Note: Boost does a much better job of limiting instantiations.
	We require VC8 so a lot of potential version checks are omitted.
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Templates/IsPointer.h"
#include "Misc/AssertionMacros.h"
#include "Templates/AndOrNot.h"
#include "Templates/EnableIf.h"
#include "Templates/IsArithmetic.h"
#include "Templates/IsEnum.h"
#include "Templates/Models.h"

#include "Templates/IsPODType.h"
#include "Templates/IsUECoreType.h"
#include "Templates/IsTriviallyCopyConstructible.h"

/*-----------------------------------------------------------------------------
 * Macros to abstract the presence of certain compiler intrinsic type traits 
 -----------------------------------------------------------------------------*/
#define HAS_TRIVIAL_CONSTRUCTOR(T) __has_trivial_constructor(T)
#define IS_POD(T) __is_pod(T)
#define IS_EMPTY(T) __is_empty(T)


/*-----------------------------------------------------------------------------
	Type traits similar to TR1 (uses intrinsics supported by VC8)
	Should be updated/revisited/discarded when compiler support for tr1 catches up.
 -----------------------------------------------------------------------------*/

/** Is type DerivedType inherited from BaseType. */
template<typename DerivedType, typename BaseType>
struct TIsDerivedFrom
{
	// Different size types so we can compare their sizes later.
	typedef char No[1];
	typedef char Yes[2];

	// Overloading Test() s.t. only calling it with something that is
	// a BaseType (or inherited from the BaseType) will return a Yes.
	static Yes& Test( BaseType* );
	static Yes& Test( const BaseType* );
	static No& Test( ... );

	// Makes a DerivedType ptr.
	static DerivedType* DerivedTypePtr(){ return nullptr ;}

	public:
	// Test the derived type pointer. If it inherits from BaseType, the Test( BaseType* ) 
	// will be chosen. If it does not, Test( ... ) will be chosen.
	static constexpr bool Value = sizeof(Test( DerivedTypePtr() )) == sizeof(Yes);

	static constexpr bool IsDerived = Value;
};

/** Gets the Nth type in a template parameter pack. N must be less than sizeof...(Types) */
template <int32 N, typename... Types>
struct TNthTypeFromParameterPack;

template <int32 N, typename T, typename... OtherTypes>
struct TNthTypeFromParameterPack<N, T, OtherTypes...>
{
	using Type = typename TNthTypeFromParameterPack<N - 1, OtherTypes...>::Type;
};

template <typename T, typename... OtherTypes>
struct TNthTypeFromParameterPack<0, T, OtherTypes...>
{
	using Type = T;
};

/**
 * TFormatSpecifier, only applies to numeric types
 */
template<typename T> 
struct TFormatSpecifier
{
	template <typename CharType = TCHAR>
	inline static constexpr decltype(auto) GetFormatSpecifier()
	{
		// Force the template instantiation to be dependent upon T so the compiler cannot automatically decide that this template can never be instantiated.
		// If the static_assert below were a constant 0 or something not dependent on T, compilers are free to detect this and fail to compile the template.
		// As specified in the C++ standard s14.6p8. A compiler is free to give a diagnostic here or not. MSVC ignores it, and clang/gcc instantiates the 
		// template and triggers the static_assert.
		static_assert(sizeof(T) < 0, "Format specifier not supported for this type."); // request for a format specifier for a type we do not know about
		return CHARTEXT(CharType, "Unknown"); \
	}
};
#define Expose_TFormatSpecifier(type, format) \
template<> \
struct TFormatSpecifier<type> \
{  \
	template <typename CharType = TCHAR> \
	UE_FORCEINLINE_HINT static constexpr decltype(auto) GetFormatSpecifier() \
	{ \
		return CHARTEXT(CharType, format); \
	} \
};

Expose_TFormatSpecifier(bool, "%i")
Expose_TFormatSpecifier(uint8, "%u")
Expose_TFormatSpecifier(uint16, "%u")
Expose_TFormatSpecifier(uint32, "%u")
Expose_TFormatSpecifier(uint64, "%llu")
Expose_TFormatSpecifier(int8, "%d")
Expose_TFormatSpecifier(int16, "%d")
Expose_TFormatSpecifier(int32, "%d")
Expose_TFormatSpecifier(int64, "%lld")
Expose_TFormatSpecifier(float, "%f")
Expose_TFormatSpecifier(double, "%f")
Expose_TFormatSpecifier(long double, "%f")
Expose_TFormatSpecifier(long, "%ld")
Expose_TFormatSpecifier(unsigned long, "%lu")


/**
 * TIsReferenceType
 */
template<typename T> struct TIsReferenceType      { enum { Value = false }; };
template<typename T> struct TIsReferenceType<T&>  { enum { Value = true  }; };
template<typename T> struct TIsReferenceType<T&&> { enum { Value = true  }; };

/**
 * TIsLValueReferenceType
 */
template<typename T> struct TIsLValueReferenceType     { enum { Value = false }; };
template<typename T> struct TIsLValueReferenceType<T&> { enum { Value = true  }; };

/**
 * TIsRValueReferenceType
 */
template<typename T> struct TIsRValueReferenceType      { enum { Value = false }; };
template<typename T> struct TIsRValueReferenceType<T&&> { enum { Value = true  }; };

/**
 * TIsFundamentalType
 */
template<typename T> 
struct TIsFundamentalType 
{ 
	enum { Value = TIsArithmetic<T>::Value || std::is_void_v<T> };
};

/**
 * TIsFunction
 *
 * Tests is a type is a function.
 */
template <typename T>
struct TIsFunction
{
	enum { Value = false };
};

template <typename RetType, typename... Params>
struct TIsFunction<RetType(Params...)>
{
	enum { Value = true };
};

/**
 * TIsZeroConstructType
 */
template<typename T> 
struct TIsZeroConstructType 
{ 
	enum { Value = TOr<TIsEnum<T>, TIsArithmetic<T>, TIsPointer<T>>::Value };
};

/**
 * TIsWeakPointerType
 */
template<typename T> 
struct TIsWeakPointerType
{ 
	enum { Value = false };
};


/**
 * TNameOf
 */
template<typename T> 
struct TNameOf
{ 
	inline static TCHAR const* GetName()
	{
		check(0); // request for the name of a type we do not know about
		return TEXT("Unknown");
	}
};

#define Expose_TNameOf(type) \
template<> \
struct TNameOf<type> \
{  \
	UE_FORCEINLINE_HINT static TCHAR const* GetName() \
	{ \
		return TEXT(#type); \
	} \
};

Expose_TNameOf(uint8)
Expose_TNameOf(uint16)
Expose_TNameOf(uint32)
Expose_TNameOf(uint64)
Expose_TNameOf(int8)
Expose_TNameOf(int16)
Expose_TNameOf(int32)
Expose_TNameOf(int64)
Expose_TNameOf(float)
Expose_TNameOf(double)

/*-----------------------------------------------------------------------------
	Call traits - Modeled somewhat after boost's interfaces.
-----------------------------------------------------------------------------*/

/**
 * Call traits helpers
 */
template <typename T, bool TypeIsSmall>
struct TCallTraitsParamTypeHelper
{
	typedef const T& ParamType;
	typedef const T& ConstParamType;
};
template <typename T>
struct TCallTraitsParamTypeHelper<T, true>
{
	typedef const T ParamType;
	typedef const T ConstParamType;
};
template <typename T>
struct TCallTraitsParamTypeHelper<T*, true>
{
	typedef T* ParamType;
	typedef const T* ConstParamType;
};


/*-----------------------------------------------------------------------------
	Helper templates for dealing with 'const' in template code
-----------------------------------------------------------------------------*/

/*-----------------------------------------------------------------------------
 * TCallTraits
 *
 * Same call traits as boost, though not with as complete a solution.
 *
 * The main member to note is ParamType, which specifies the optimal 
 * form to pass the type as a parameter to a function.
 * 
 * Has a small-value optimization when a type is a POD type and as small as a pointer.
-----------------------------------------------------------------------------*/

/**
 * base class for call traits. Used to more easily refine portions when specializing
 */
template <typename T>
struct TCallTraitsBase
{
private:
	enum { PassByValue = TOr<TAndValue<(sizeof(T) <= sizeof(void*)), TIsPODType<T>>, TIsArithmetic<T>>::Value };

public:
	typedef T ValueType;
	typedef T& Reference;
	typedef const T& ConstReference;
	typedef typename TCallTraitsParamTypeHelper<T, PassByValue>::ParamType ParamType;
	typedef typename TCallTraitsParamTypeHelper<T, PassByValue>::ConstParamType ConstPointerType;
};

/**
 * TCallTraits
 */
template <typename T>
struct TCallTraits : public TCallTraitsBase<T> {};

// Fix reference-to-reference problems.
template <typename T>
struct TCallTraits<T&>
{
	typedef T& ValueType;
	typedef T& Reference;
	typedef const T& ConstReference;
	typedef T& ParamType;
	typedef T& ConstPointerType;
};

// Array types
template <typename T, size_t N>
struct TCallTraits<T [N]>
{
private:
	typedef T ArrayType[N];
public:
	typedef const T* ValueType;
	typedef ArrayType& Reference;
	typedef const ArrayType& ConstReference;
	typedef const T* const ParamType;
	typedef const T* const ConstPointerType;
};

// const array types
template <typename T, size_t N>
struct TCallTraits<const T [N]>
{
private:
	typedef const T ArrayType[N];
public:
	typedef const T* ValueType;
	typedef ArrayType& Reference;
	typedef const ArrayType& ConstReference;
	typedef const T* const ParamType;
	typedef const T* const ConstPointerType;
};


/*-----------------------------------------------------------------------------
	Traits for our particular container classes
-----------------------------------------------------------------------------*/

/**
 * Helper for array traits. Provides a common base to more easily refine a portion of the traits
 * when specializing. Mainly used by MemoryOps.h which is used by the contiguous storage containers like TArray.
 */
template<typename T>
struct TTypeTraitsBase
{
	typedef typename TCallTraits<T>::ParamType ConstInitType;
	typedef typename TCallTraits<T>::ConstPointerType ConstPointerType;

	// There's no good way of detecting this so we'll just assume it to be true for certain known types and expect
	// users to customize it for their custom types.
	enum { IsBytewiseComparable = TOr<TIsEnum<T>, TIsArithmetic<T>, TIsPointer<T>>::Value };
};

/**
 * Traits for types.
 */
template<typename T> struct TTypeTraits : public TTypeTraitsBase<T> {};


struct FVirtualDestructor
{
	virtual ~FVirtualDestructor() = default;
};

template <typename T, typename U>
struct TMoveSupportTraitsBase
{
	// Param type is not an const lvalue reference, which means it's pass-by-value, so we should just provide a single type for copying.
	// Move overloads will be ignored due to SFINAE.
	typedef U Copy;
};

template <typename T>
struct TMoveSupportTraitsBase<T, const T&>
{
	// Param type is a const lvalue reference, so we can provide an overload for moving.
	typedef const T& Copy;
	typedef T&&      Move;
};

/**
 * This traits class is intended to be used in pairs to allow efficient and correct move-aware overloads for generic types.
 * For example:
 *
 * template <typename T>
 * void Func(typename TMoveSupportTraits<T>::Copy Obj)
 * {
 *     // Copy Obj here
 * }
 *
 * template <typename T>
 * void Func(typename TMoveSupportTraits<T>::Move Obj)
 * {
 *     // Move from Obj here as if it was passed as T&&
 * }
 *
 * Structuring things in this way will handle T being a pass-by-value type (e.g. ints, floats, other 'small' types) which
 * should never have a reference overload.
 */
template <typename T>
struct TMoveSupportTraits : TMoveSupportTraitsBase<T, typename TCallTraits<T>::ParamType>
{
};

/**
 * Tests if a type T is bitwise-constructible from a given argument type U.  That is, whether or not
 * the U can be memcpy'd in order to produce an instance of T, rather than having to go
 * via a constructor.
 *
 * Examples:
 * TIsBitwiseConstructible<PODType,    PODType   >::Value == true  // PODs can be trivially copied
 * TIsBitwiseConstructible<const int*, int*      >::Value == true  // a non-const Derived pointer is trivially copyable as a const Base pointer
 * TIsBitwiseConstructible<int*,       const int*>::Value == false // not legal the other way because it would be a const-correctness violation
 * TIsBitwiseConstructible<int32,      uint32    >::Value == true  // signed integers can be memcpy'd as unsigned integers
 * TIsBitwiseConstructible<uint32,     int32     >::Value == true  // and vice versa
 */

template <typename T, typename Arg>
struct TIsBitwiseConstructible
{
	static_assert(
		!TIsReferenceType<T  >::Value &&
		!TIsReferenceType<Arg>::Value,
		"TIsBitwiseConstructible is not designed to accept reference types");

	static_assert(
		std::is_same_v<T,   std::remove_cv_t<T  >> &&
		std::is_same_v<Arg, std::remove_cv_t<Arg>>,
		"TIsBitwiseConstructible is not designed to accept qualified types");

	// Assume no bitwise construction in general
	enum { Value = false };
};

template <typename T>
struct TIsBitwiseConstructible<T, T>
{
	// Ts can always be bitwise constructed from itself if it is trivially copyable.
	enum { Value = TIsTriviallyCopyConstructible<T>::Value };
};

template <typename T, typename U>
struct TIsBitwiseConstructible<const T, U> : TIsBitwiseConstructible<T, U>
{
	// Constructing a const T is the same as constructing a T
};

// Const pointers can be bitwise constructed from non-const pointers.
// This is not true for pointer conversions in general, e.g. where an offset may need to be applied in the case
// of multiple inheritance, but there is no way of detecting that at compile-time.
template <typename T>
struct TIsBitwiseConstructible<const T*, T*>
{
	// Constructing a const T is the same as constructing a T
	enum { Value = true };
};

// Unsigned types can be bitwise converted to their signed equivalents, and vice versa.
// (assuming two's-complement, which we are)
template <> struct TIsBitwiseConstructible<uint8,   int8>  { enum { Value = true }; };
template <> struct TIsBitwiseConstructible< int8,  uint8>  { enum { Value = true }; };
template <> struct TIsBitwiseConstructible<uint16,  int16> { enum { Value = true }; };
template <> struct TIsBitwiseConstructible< int16, uint16> { enum { Value = true }; };
template <> struct TIsBitwiseConstructible<uint32,  int32> { enum { Value = true }; };
template <> struct TIsBitwiseConstructible< int32, uint32> { enum { Value = true }; };
template <> struct TIsBitwiseConstructible<uint64,  int64> { enum { Value = true }; };
template <> struct TIsBitwiseConstructible< int64, uint64> { enum { Value = true }; };

// ANSICHAR can be bitwise converted to a byte or UTF8CHAR
// conversions in the other direction (to ANSICHAR) are not generally safe
template <> struct TIsBitwiseConstructible< int8,    ANSICHAR> { enum { Value = true }; };
template <> struct TIsBitwiseConstructible<uint8,    ANSICHAR> { enum { Value = true }; };
template <> struct TIsBitwiseConstructible<UTF8CHAR, ANSICHAR> { enum { Value = true }; };

// UTF8CHAR can be bitwise converted to a byte
// conversions in the other direction (to UTF8CHAR) are not generally safe
template <> struct TIsBitwiseConstructible< int8, UTF8CHAR> { enum { Value = true }; };
template <> struct TIsBitwiseConstructible<uint8, UTF8CHAR> { enum { Value = true }; };

#define GENERATE_MEMBER_FUNCTION_CHECK(MemberName, Result, ConstModifier, ...)									\
template <typename T>																							\
class THasMemberFunction_##MemberName																			\
{																												\
	template <typename U, Result(U::*)(__VA_ARGS__) ConstModifier> struct Check;								\
	template <typename U> static char MemberTest(Check<U, &U::MemberName> *);									\
	template <typename U> static int MemberTest(...);															\
public:																											\
	enum { Value = sizeof(MemberTest<T>(nullptr)) == sizeof(char) };											\
};

/*-----------------------------------------------------------------------------
 * Undef Macros abstracting the presence of certain compiler intrinsic type traits
 -----------------------------------------------------------------------------*/
#undef IS_EMPTY
#undef IS_POD
#undef HAS_TRIVIAL_CONSTRUCTOR
