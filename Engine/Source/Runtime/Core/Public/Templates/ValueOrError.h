// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include <type_traits>

template <typename... ArgTypes>
struct TValueOrError_ValueProxy
{
	[[nodiscard]] explicit TValueOrError_ValueProxy(ArgTypes&&... InArgs UE_LIFETIMEBOUND)
		: Args(Forward<ArgTypes>(InArgs)...)
	{
	}

	TTuple<ArgTypes&&...> Args;
};

template <typename... ArgTypes>
struct TValueOrError_ErrorProxy
{
	[[nodiscard]] explicit TValueOrError_ErrorProxy(ArgTypes&&... InArgs UE_LIFETIMEBOUND)
		: Args(Forward<ArgTypes>(InArgs)...)
	{
	}

	TTuple<ArgTypes&&...> Args;
};

template <typename... ArgTypes>
[[nodiscard]] UE_REWRITE TValueOrError_ValueProxy<ArgTypes...> MakeValue(ArgTypes&&... Args UE_LIFETIMEBOUND)
{
	return TValueOrError_ValueProxy<ArgTypes...>(Forward<ArgTypes>(Args)...);
}

template <typename... ArgTypes>
[[nodiscard]] UE_REWRITE TValueOrError_ErrorProxy<ArgTypes...> MakeError(ArgTypes&&... Args UE_LIFETIMEBOUND)
{
	return TValueOrError_ErrorProxy<ArgTypes...>(Forward<ArgTypes>(Args)...);
}

/**
 * Type used to return either a value or an error.
 *
 * These must have a value or an error when newly constructed, but it is possible to have neither
 * because of the functions to steal the value or error. This is critical for callers to consider
 * since it means that HasValue() and HasError() must be checked independently; a return value of
 * false from one does not imply that the other will return true.
 *
 * The MakeValue and MakeError functions may be used to construct these conveniently.
 */
template <typename ValueType, typename ErrorType>
class TValueOrError
{
	static_assert(!std::is_reference_v<ValueType> && !std::is_reference_v<ErrorType>, "ValueType and ErrorType cannot be references");

	/** Wrap the error type to allow ValueType and ErrorType to be the same type. */
	struct FWrapErrorType
	{
		template <typename... ArgTypes>
		FWrapErrorType(ArgTypes&&... Args) : Error(Forward<ArgTypes>(Args)...) {}
		ErrorType Error;
	};

	/** A unique empty type used in the unset state after stealing the value or error. */
	struct FEmptyType
	{
	};

	template <typename... ArgTypes, uint32... ArgIndices>
	[[nodiscard]] inline TValueOrError(TValueOrError_ValueProxy<ArgTypes...>&& Proxy, TIntegerSequence<uint32, ArgIndices...>)
		: Variant(TInPlaceType<ValueType>(), MoveTemp(Proxy.Args).template Get<ArgIndices>()...)
	{
	}

	template <typename... ArgTypes, uint32... ArgIndices>
	[[nodiscard]] inline TValueOrError(TValueOrError_ErrorProxy<ArgTypes...>&& Proxy, TIntegerSequence<uint32, ArgIndices...>)
		: Variant(TInPlaceType<FWrapErrorType>(), MoveTemp(Proxy.Args).template Get<ArgIndices>()...)
	{
	}

public:
	/** Construct the value from a proxy from MakeValue. */
	template <typename... ArgTypes>
	[[nodiscard]] inline TValueOrError(TValueOrError_ValueProxy<ArgTypes...>&& Proxy)
		: TValueOrError(MoveTemp(Proxy), TMakeIntegerSequence<uint32, sizeof...(ArgTypes)>())
	{
	}

	/** Construct the error from a proxy from MakeError. */
	template <typename... ArgTypes>
	[[nodiscard]] inline TValueOrError(TValueOrError_ErrorProxy<ArgTypes...>&& Proxy)
		: TValueOrError(MoveTemp(Proxy), TMakeIntegerSequence<uint32, sizeof...(ArgTypes)>())
	{
	}

	/** Check whether a value is set. Prefer HasValue and HasError to this. !IsValid() does *not* imply HasError(). */
	[[nodiscard]] UE_REWRITE bool IsValid() const
	{
		return Variant.template IsType<ValueType>();
	}

	/** Whether the error is set. An error does imply no value. No error does *not* imply that a value is set. */
	[[nodiscard]] UE_REWRITE bool HasError() const
	{
		return Variant.template IsType<FWrapErrorType>();
	}

	/** Access the error. Asserts if this does not have an error. */
	[[nodiscard]] inline ErrorType& GetError() & UE_LIFETIMEBOUND
	{
		return Variant.template Get<FWrapErrorType>().Error;
	}
	[[nodiscard]] UE_REWRITE const ErrorType& GetError() const & UE_LIFETIMEBOUND
	{
		return const_cast<TValueOrError*>(this)->GetError();
	}
	[[nodiscard]] UE_REWRITE ErrorType&& GetError() && UE_LIFETIMEBOUND
	{
		return MoveTempIfPossible(this->GetError());
	}

	/** Access the error if it is set. */
	[[nodiscard]] inline ErrorType* TryGetError() UE_LIFETIMEBOUND
	{
		if (FWrapErrorType* Wrap = Variant.template TryGet<FWrapErrorType>())
		{
			return &Wrap->Error;
		}
		return nullptr;
	}
	[[nodiscard]] UE_REWRITE const ErrorType* TryGetError() const UE_LIFETIMEBOUND
	{
		return const_cast<TValueOrError*>(this)->TryGetError();
	}

	/** Steal the error. Asserts if this does not have an error. This causes the error to be unset. */
	[[nodiscard]] inline ErrorType StealError()
	{
		ErrorType Temp = MoveTempIfPossible(GetError());
		Variant.template Emplace<FEmptyType>();
		return Temp;
	}

	/** Whether the value is set. A value does imply no error. No value does *not* imply that an error is set. */
	[[nodiscard]] UE_REWRITE bool HasValue() const
	{
		return Variant.template IsType<ValueType>();
	}

	/** Access the value. Asserts if this does not have a value. */
	[[nodiscard]] inline ValueType& GetValue() & UE_LIFETIMEBOUND
	{
		return Variant.template Get<ValueType>();
	}
	[[nodiscard]] UE_REWRITE const ValueType& GetValue() const & UE_LIFETIMEBOUND
	{
		return const_cast<TValueOrError*>(this)->GetValue();
	}
	[[nodiscard]] UE_REWRITE ValueType&& GetValue() && UE_LIFETIMEBOUND
	{
		return MoveTempIfPossible(this->GetValue());
	}

	/** Access the value if it is set. */
	[[nodiscard]] UE_REWRITE ValueType* TryGetValue() UE_LIFETIMEBOUND
	{
		return Variant.template TryGet<ValueType>();
	}
	[[nodiscard]] UE_REWRITE const ValueType* TryGetValue() const UE_LIFETIMEBOUND
	{
		return const_cast<TValueOrError*>(this)->TryGetValue();
	}

	/** Steal the value. Asserts if this does not have a value. This causes the value to be unset. */
	[[nodiscard]] inline ValueType StealValue()
	{
		ValueType Temp = MoveTempIfPossible(GetValue());
		Variant.template Emplace<FEmptyType>();
		return Temp;
	}

private:
	TVariant<ValueType, FWrapErrorType, FEmptyType> Variant;
};

template <typename ValueType>
class TValueOrError<ValueType, void>
{
	static_assert(!std::is_reference_v<ValueType>, "ValueType cannot be a reference");

	template <typename... ArgTypes, uint32... ArgIndices>
	[[nodiscard]] inline TValueOrError(TValueOrError_ValueProxy<ArgTypes...>&& Proxy, TIntegerSequence<uint32, ArgIndices...>)
		: Value(InPlace, MoveTemp(Proxy.Args).template Get<ArgIndices>()...)
	{
	}

public:
	/** Construct the value from a proxy from MakeValue. */
	template <typename... ArgTypes>
	[[nodiscard]] inline TValueOrError(TValueOrError_ValueProxy<ArgTypes...>&& Proxy)
		: TValueOrError(MoveTemp(Proxy), TMakeIntegerSequence<uint32, sizeof...(ArgTypes)>())
	{
	}

	/** Construct the error from a proxy from MakeError. */
	[[nodiscard]] inline TValueOrError(TValueOrError_ErrorProxy<>&& Proxy)
	{
	}

	/** Whether the error is set. An error does imply no value. */
	[[nodiscard]] UE_REWRITE bool HasError() const
	{
		return !Value.IsSet();
	}

	/** Whether the value is set. A value does imply no error. */
	[[nodiscard]] UE_REWRITE bool HasValue() const
	{
		return Value.IsSet();
	}

	/** Access the value. Asserts if this does not have a value. */
	[[nodiscard]] inline ValueType& GetValue() & UE_LIFETIMEBOUND
	{
		return Value.GetValue();
	}
	[[nodiscard]] UE_REWRITE const ValueType& GetValue() const & UE_LIFETIMEBOUND
	{
		return const_cast<TValueOrError*>(this)->GetValue();
	}
	[[nodiscard]] UE_REWRITE ValueType&& GetValue() && UE_LIFETIMEBOUND
	{
		return MoveTempIfPossible(this->GetValue());
	}

	/** Access the value if it is set. */
	[[nodiscard]] inline ValueType* TryGetValue() UE_LIFETIMEBOUND
	{
		return Value.GetPtrOrNull();
	}
	[[nodiscard]] UE_REWRITE const ValueType* TryGetValue() const UE_LIFETIMEBOUND
	{
		return const_cast<TValueOrError*>(this)->TryGetValue();
	}

	/** Steal the value. Asserts if this does not have a value. This causes the value to be unset. */
	[[nodiscard]] inline ValueType StealValue()
	{
		ValueType Temp = MoveTempIfPossible(GetValue());
		Value.Reset();
		return Temp;
	}

private:
	TOptional<ValueType> Value;
};

template <typename ErrorType>
class TValueOrError<void, ErrorType>
{
	static_assert(!std::is_reference_v<ErrorType>, "ErrorType cannot be a reference");

	template <typename... ArgTypes, uint32... ArgIndices>
	[[nodiscard]] inline TValueOrError(TValueOrError_ErrorProxy<ArgTypes...>&& Proxy, TIntegerSequence<uint32, ArgIndices...>)
		: Error(InPlace, MoveTemp(Proxy.Args).template Get<ArgIndices>()...)
	{
	}

public:
	/** Construct the value from a proxy from MakeValue. */
	[[nodiscard]] inline TValueOrError(TValueOrError_ValueProxy<>&& Proxy)
	{
	}

	/** Construct the error from a proxy from MakeError. */
	template <typename... ArgTypes>
	[[nodiscard]] inline TValueOrError(TValueOrError_ErrorProxy<ArgTypes...>&& Proxy)
		: TValueOrError(MoveTemp(Proxy), TMakeIntegerSequence<uint32, sizeof...(ArgTypes)>())
	{
	}

	/** Whether the value is set. A value does imply no error. */
	[[nodiscard]] UE_REWRITE bool HasValue() const
	{
		return !Error.IsSet();
	}

	/** Whether the error is set. An error does imply no value. */
	[[nodiscard]] UE_REWRITE bool HasError() const
	{
		return Error.IsSet();
	}

	/** Access the error. Asserts if this does not have an error. */
	[[nodiscard]] inline ErrorType& GetError() & UE_LIFETIMEBOUND
	{
		return Error.GetValue();
	}
	[[nodiscard]] UE_REWRITE const ErrorType& GetError() const & UE_LIFETIMEBOUND
	{
		return const_cast<TValueOrError*>(this)->GetError();
	}
	[[nodiscard]] UE_REWRITE ErrorType&& GetError() && UE_LIFETIMEBOUND
	{
		return MoveTempIfPossible(this->GetError());
	}

	/** Access the error if it is set. */
	[[nodiscard]] inline ErrorType* TryGetError() UE_LIFETIMEBOUND
	{
		return Error.GetPtrOrNull();
	}
	[[nodiscard]] UE_REWRITE const ErrorType* TryGetError() const UE_LIFETIMEBOUND
	{
		return const_cast<TValueOrError*>(this)->TryGetError();
	}

	/** Steal the error. Asserts if this does not have an error. This causes the error to be unset. */
	[[nodiscard]] inline ErrorType StealError()
	{
		ErrorType Temp = MoveTempIfPossible(GetError());
		Error.Reset();
		return Temp;
	}

private:
	TOptional<ErrorType> Error;
};

template <>
class TValueOrError<void, void>
{
public:
	/** Construct the value from a proxy from MakeValue. */
	[[nodiscard]] inline TValueOrError(TValueOrError_ValueProxy<>&& Proxy)
		: bValue(true)
	{
	}

	/** Construct the error from a proxy from MakeError. */
	[[nodiscard]] inline TValueOrError(TValueOrError_ErrorProxy<>&& Proxy)
		: bValue(false)
	{
	}

	/** Whether the error is set. An error does imply no value. */
	[[nodiscard]] UE_REWRITE bool HasError() const
	{
		return !bValue;
	}

	/** Whether the value is set. A value does imply no error. */
	[[nodiscard]] UE_REWRITE bool HasValue() const
	{
		return bValue;
	}

private:
	bool bValue;
};
