// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_LOW_LEVEL_TESTS
#include "TestMacros/Assertions.h" // HEADER_UNIT_IGNORE
#elif defined(WITH_AUTOMATION_TESTS) || (WITH_DEV_AUTOMATION_TESTS || WITH_PERF_AUTOMATION_TESTS)
#include "Misc/AssertionMacros.h"
#include "Misc/AutomationTest.h"
#include "Tests/EnsureScope.h"
#include "Tests/CheckScope.h"

//requires that an UE `ensure` fails in this call
#define REQUIRE_ENSURE(...) INTERNAL_UE_ENSURE( "REQUIRE_ENSURE", DO_ENSURE, #__VA_ARGS__, __VA_ARGS__ )

//requires that an UE `ensure` fails with a message that matches the supplied message
#define REQUIRE_ENSURE_MSG(msg, ...) INTERNAL_UE_ENSURE_MSG(msg, "REQUIRE_ENSURE", DO_ENSURE, #__VA_ARGS__, __VA_ARGS__ )

//checks that an UE `ensure` fails in this call
#define CHECK_ENSURE(...) INTERNAL_UE_ENSURE( "CHECK_ENSURE", DO_ENSURE, Catch::ResultDisposition::ContinueOnFailure, #__VA_ARGS__, __VA_ARGS__ )

//checks that an UE `ensure` fails with a message that matches the supplied message
#define CHECK_ENSURE_MSG(msg, ...) INTERNAL_UE_ENSURE_MSG(msg, "CHECK_ENSURE", DO_ENSURE, Catch::ResultDisposition::ContinueOnFailure, #__VA_ARGS__, __VA_ARGS__ )

//requires that a UE `check` fails in this call
#define REQUIRE_CHECK(...) INTERNAL_UE_CHECK( "REQUIRE_CHECK", DO_CHECK, #__VA_ARGS__, __VA_ARGS__ )

//requires that a UE `check` fails in this call contains the supplies message
#define REQUIRE_CHECK_MSG(msg, ...) INTERNAL_UE_CHECK_MSG(msg, "REQUIRE_CHECK", DO_CHECK, #__VA_ARGS__, __VA_ARGS__ )

//requires that a UE `checkSlow` fails in this call
#define REQUIRE_CHECK_SLOW(...) INTERNAL_UE_CHECK( "REQUIRE_CHECK_SLOW", DO_GUARD_SLOW, #__VA_ARGS__, __VA_ARGS__ )

//requires that a UE `checkSlow` fails in this call contains the supplies message
#define REQUIRE_CHECK_SLOW_MSG(msg, ...) INTERNAL_UE_CHECK_MSG(msg, "REQUIRE_CHECK_SLOW", DO_GUARD_SLOW, #__VA_ARGS__, __VA_ARGS__ )

#define INTERNAL_UE_ENSURE( macroName, doEnsure, ensureExpr, ... ) \
	do { \
		FEnsureScope scope; \
		static_cast<void>(__VA_ARGS__); \
		bool bEncounteredEnsure = scope.GetCount() > 0; \
		if (doEnsure && !bEncounteredEnsure) \
			FAutomationTestFramework::Get().GetCurrentTest()->AddError(TEXT("Expected failure of `ensure` not received.")); \
	} while(false) \

#define INTERNAL_UE_ENSURE_MSG(msg, macroName, doEnsure, ensureExpr, ... ) \
	do { \
		FEnsureScope scope(msg); \
		static_cast<void>(__VA_ARGS__); \
		bool bEncounteredEnsure = scope.GetCount() > 0; \
		if (doEnsure && !bEncounteredEnsure) \
			FAutomationTestFramework::Get().GetCurrentTest()->AddError(TEXT("Expected failure of `ensure` with message %s not received", msg)); \
	} while(false) \

#define INTERNAL_UE_CHECK(macroName, doCheck, checkExpr, ... ) \
	do { \
		FCheckScope scope; \
		(void)(__VA_ARGS__); \
		bool bEncounteredEnsure = scope.GetCount() > 0; \
		if (doCheck && !bEncounteredEnsure) \
			FAutomationTestFramework::Get().GetCurrentTest()->AddError(TEXT("Expected failure of `check` not received")); \
	} while(false) \

#define INTERNAL_UE_CHECK_MSG(msg, macroName, doCheck, checkExpr, ... ) \
	do { \
		FCheckScope scope(msg); \
		(void)(__VA_ARGS__); \
		bool bEncounteredEnsure = scope.GetCount() > 0; \
		if (doCheck && !bEncounteredEnsure) \
			FAutomationTestFramework::Get().GetCurrentTest()->AddError(TEXT("Expected failure of `check` containing message %s not received", msg)); \
	} while(false) \

#endif