// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "HAL/PlatformTime.h"
#include "Math/NumericLimits.h"

template<uint32 NumRuns, typename TestT>
void Benchmark(const TCHAR* TestName, TestT&& TestBody)
{
	UE_LOGF(LogTemp, Log, "\n-------------------------------\n%ls", TestName);
	double MinTime = TNumericLimits<double>::Max();
	double TotalTime = 0;
	for (uint32 RunNo = 0; RunNo != NumRuns; ++RunNo)
	{
		double Time = FPlatformTime::Seconds();
		TestBody();
		Time = FPlatformTime::Seconds() - Time;

		UE_LOGF(LogTemp, Log, "#%d: %f secs", RunNo, Time);

		TotalTime += Time;
		if (MinTime > Time)
		{
			MinTime = Time;
		}
	}
	UE_LOGF(LogTemp, Log, "min: %f secs, avg: %f secs\n-------------------------------\n", MinTime, TotalTime / NumRuns);

#if NO_LOGGING
	printf("%s\nmin: %f secs, avg: %f secs\n-------------------------------\n\n", TCHAR_TO_ANSI(TestName), MinTime, TotalTime / NumRuns);
#endif
}

#define UE_BENCHMARK(NumRuns, ...) { TRACE_CPUPROFILER_EVENT_SCOPE_STR(TEXT(#__VA_ARGS__)); Benchmark<NumRuns>(TEXT(#__VA_ARGS__), __VA_ARGS__); }

