// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HAL/Platform.h"

class FCheckScopeOutputDeviceError;

///@brief Scope that captures failed `check` calls
struct FCheckScope
{
	CORE_API FCheckScope();

	///@brief Only catches failed `check` calls that contain Msg
	///@param Msg string to look for in the `check` message
	CORE_API explicit FCheckScope(const ANSICHAR* Msg);
	CORE_API ~FCheckScope();

	CORE_API int GetCount();
private:
	FCheckScopeOutputDeviceError* DeviceError;
	bool bIgnoreDebugger;
	bool bCriticalError;
};