// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Templates/Function.h"

struct FEnsureHandlerArgs;

///@brief Scope that captures failed `ensure` calls
struct FEnsureScope
{
	CORE_API FEnsureScope();

	///@brief Only captures failed `ensure` calls with the exact message 
	///@param Msg string to compare against failed `ensure` message
	CORE_API explicit FEnsureScope(const ANSICHAR* Msg);

	///@brief Captures failed `ensure` calls that return true from callback
	///@param EnsureFunc lambda called for each failed `ensure` return true to handle the error
	CORE_API explicit FEnsureScope(TFunction<bool(const FEnsureHandlerArgs&)> EnsureFunc);
	CORE_API ~FEnsureScope();

	int GetCount()
	{
		return Count;
	}

private:
	TFunction<bool(const FEnsureHandlerArgs&)> OldHandler;
	int Count;
};