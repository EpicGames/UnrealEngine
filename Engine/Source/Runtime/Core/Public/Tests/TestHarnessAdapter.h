// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Common test harness include file. Use this with tests that can be run as both low level tests or functional tests.
 * The two types of frameworks are as follows:
 * - Low level tests using Catch2, generates a monolithic test application, built separate
 * - UE legacy test automation framework, tests are built into the target executable
 */

#if WITH_LOW_LEVEL_TESTS
#include "TestHarness.h" // HEADER_UNIT_IGNORE
#elif defined(WITH_AUTOMATION_TESTS) || (WITH_DEV_AUTOMATION_TESTS || WITH_PERF_AUTOMATION_TESTS)
#include "Misc/AutomationTest.h" // IWYU pragma: export
#include "Misc/LowLevelTestAdapter.h" // IWYU pragma: export
#endif

#define CHECK_AND_SET_ERROR_ON_FAIL(What, Value, Error) do { \
	Error = Error || (!(Value)); \
	CHECK_MESSAGE(What, Value); \
} while (false)
