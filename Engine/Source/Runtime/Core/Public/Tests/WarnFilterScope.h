// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"
#include "HAL/Platform.h"
#include "Misc/ScopeExit.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"

class FFeedbackContext;
struct FFilterFeedback;

/**
 * Scope that captures and optionally filters display/warning/error log messages.
 *
 * Filtering expected warnings and errors during tests is necessary to avoid Horde flagging them as failures.
 */
struct FWarnFilterScope
{
	/**
	 * @param LogHandler Function to handle log messages. Returning true filters the message from the log output.
	 */
	explicit FWarnFilterScope(TUniqueFunction<bool(const TCHAR* Message, ELogVerbosity::Type Verbosity, const FName& Category)> LogHandler);

	FWarnFilterScope(const FWarnFilterScope&) = delete;
	FWarnFilterScope& operator=(const FWarnFilterScope&) = delete;

	~FWarnFilterScope();

private:
	FFeedbackContext* OldWarn;
	FFilterFeedback* Feedback;
};

/**
 * Asserts that this exact log occurs within the scope of this statement.
 *
 * Supported by the LowLevelTests style of test code.
 */
#define CHECK_LOG_SCOPE(ExpectedCategory, ExpectedVerbosity, ExpectedMessage) \
	static_assert( \
		::ELogVerbosity::ExpectedVerbosity == ::ELogVerbosity::Error || \
		::ELogVerbosity::ExpectedVerbosity == ::ELogVerbosity::Warning || \
		::ELogVerbosity::ExpectedVerbosity == ::ELogVerbosity::Display, \
		"Only Error/Warning/Display verbosity is supported by CHECK_LOG_SCOPE."); \
	bool UE_JOIN(bLogScopeLogged, __LINE__) = false; \
	::FWarnFilterScope UE_JOIN(LogScope, __LINE__)([&UE_JOIN(bLogScopeLogged, __LINE__)](const TCHAR* Message, ::ELogVerbosity::Type Verbosity, const FName& Category) \
	{ \
		if (Category == #ExpectedCategory && Verbosity == ::ELogVerbosity::ExpectedVerbosity && Message == TEXTVIEW(ExpectedMessage)) \
		{ \
			UE_JOIN(bLogScopeLogged, __LINE__) = true; \
			return true; \
		} \
		return false; \
	}); \
	ON_SCOPE_EXIT \
	{ \
		if (!UE_JOIN(bLogScopeLogged, __LINE__)) \
		{ \
			FAIL_CHECK("Missing expected log message. Inspect the log for similar messages. Expected:\n" #ExpectedCategory ": " #ExpectedVerbosity ": " ExpectedMessage); \
		} \
	}
