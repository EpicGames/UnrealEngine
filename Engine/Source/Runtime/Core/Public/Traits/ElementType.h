// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include <initializer_list>
#include <type_traits>

namespace UE::Core::Private::ElementType
{

template <typename T, typename = void>
struct TImpl
{
};

template <typename T>
struct TImpl<T, std::void_t<typename T::ElementType>>
{
	using Type = typename T::ElementType;
};

} // UE::Core::Private::ElementType

/**
 * Traits class which gets the element type of a container.
 */
template <typename T>
struct TElementType : UE::Core::Private::ElementType::TImpl<T>
{
};

template <typename T> struct TElementType<             T& > : TElementType<T> {};
template <typename T> struct TElementType<             T&&> : TElementType<T> {};
template <typename T> struct TElementType<const          T> : TElementType<T> {};
template <typename T> struct TElementType<      volatile T> : TElementType<T> {};
template <typename T> struct TElementType<const volatile T> : TElementType<T> {};

/**
 * Specialization for C arrays
 */
template <typename T, size_t N> struct TElementType<               T[N]> { using Type = T; };
template <typename T, size_t N> struct TElementType<const          T[N]> { using Type = T; };
template <typename T, size_t N> struct TElementType<      volatile T[N]> { using Type = T; };
template <typename T, size_t N> struct TElementType<const volatile T[N]> { using Type = T; };

/**
 * Specialization for initializer lists
 */
template <typename T>
struct TElementType<std::initializer_list<T>>
{
	using Type = std::remove_cv_t<T>;
};

template <typename T>
using TElementType_T = typename TElementType<T>::Type;
