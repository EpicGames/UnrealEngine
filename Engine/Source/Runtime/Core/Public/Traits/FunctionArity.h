// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Type trait which yields the number of arguments in a function type, e.g.:
 *
 * TFunctionArity_V<int(float, bool, const char*)> == 3
 *
 * This doesn't work for pointers or references to functions, or member function pointer types  - use the correct one of these to get a pure function type:
 * - std::remove_pointer_t
 * - std::remove_reference_t
 * - TRemoveMemberPtr_T
 */
template <typename FunctionType>
struct TFunctionArity
{
};

template <typename RetType, typename... ParamTypes> struct TFunctionArity<RetType(ParamTypes...)                 > { inline static constexpr unsigned int Value = sizeof...(ParamTypes); };
template <typename RetType, typename... ParamTypes> struct TFunctionArity<RetType(ParamTypes...)               & > { inline static constexpr unsigned int Value = sizeof...(ParamTypes); };
template <typename RetType, typename... ParamTypes> struct TFunctionArity<RetType(ParamTypes...)               &&> { inline static constexpr unsigned int Value = sizeof...(ParamTypes); };
template <typename RetType, typename... ParamTypes> struct TFunctionArity<RetType(ParamTypes...) const           > { inline static constexpr unsigned int Value = sizeof...(ParamTypes); };
template <typename RetType, typename... ParamTypes> struct TFunctionArity<RetType(ParamTypes...) const         & > { inline static constexpr unsigned int Value = sizeof...(ParamTypes); };
template <typename RetType, typename... ParamTypes> struct TFunctionArity<RetType(ParamTypes...) const         &&> { inline static constexpr unsigned int Value = sizeof...(ParamTypes); };
template <typename RetType, typename... ParamTypes> struct TFunctionArity<RetType(ParamTypes...)       volatile  > { inline static constexpr unsigned int Value = sizeof...(ParamTypes); };
template <typename RetType, typename... ParamTypes> struct TFunctionArity<RetType(ParamTypes...)       volatile& > { inline static constexpr unsigned int Value = sizeof...(ParamTypes); };
template <typename RetType, typename... ParamTypes> struct TFunctionArity<RetType(ParamTypes...)       volatile&&> { inline static constexpr unsigned int Value = sizeof...(ParamTypes); };
template <typename RetType, typename... ParamTypes> struct TFunctionArity<RetType(ParamTypes...) const volatile  > { inline static constexpr unsigned int Value = sizeof...(ParamTypes); };
template <typename RetType, typename... ParamTypes> struct TFunctionArity<RetType(ParamTypes...) const volatile& > { inline static constexpr unsigned int Value = sizeof...(ParamTypes); };
template <typename RetType, typename... ParamTypes> struct TFunctionArity<RetType(ParamTypes...) const volatile&&> { inline static constexpr unsigned int Value = sizeof...(ParamTypes); };

template <typename FunctionType>
inline constexpr unsigned int TFunctionArity_V = TFunctionArity<FunctionType>::Value;
