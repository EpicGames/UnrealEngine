// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Type trait which yields the type of a parameter at a particular index of a function type, e.g.:
 *
 * TFunctionParameterType_T<int(float, bool, const char*), 1> == bool
 *
 * This doesn't work for pointers or references to functions, or member function pointer types  - use the correct one of these to get a pure function type:
 * - std::remove_pointer_t
 * - std::remove_reference_t
 * - TRemoveMemberPtr_T
 */
template <typename FunctionType, unsigned int Index>
struct TFunctionParameterType
{
};

template <typename RetType, typename Param0, typename... ParamTypes, unsigned int Index>
struct TFunctionParameterType<RetType(Param0, ParamTypes...), Index> : TFunctionParameterType<RetType(ParamTypes...), Index - 1u>
{
};

template <typename RetType, typename Param0, typename... ParamTypes>
struct TFunctionParameterType<RetType(Param0, ParamTypes...), 0u>
{
	using Type = Param0;
};

template <typename RetType, typename... ParamTypes, unsigned int Index> struct TFunctionParameterType<RetType(ParamTypes...)               & , Index> : TFunctionParameterType<RetType(ParamTypes...), Index> {};
template <typename RetType, typename... ParamTypes, unsigned int Index> struct TFunctionParameterType<RetType(ParamTypes...)               &&, Index> : TFunctionParameterType<RetType(ParamTypes...), Index> {};
template <typename RetType, typename... ParamTypes, unsigned int Index> struct TFunctionParameterType<RetType(ParamTypes...) const           , Index> : TFunctionParameterType<RetType(ParamTypes...), Index> {};
template <typename RetType, typename... ParamTypes, unsigned int Index> struct TFunctionParameterType<RetType(ParamTypes...) const         & , Index> : TFunctionParameterType<RetType(ParamTypes...), Index> {};
template <typename RetType, typename... ParamTypes, unsigned int Index> struct TFunctionParameterType<RetType(ParamTypes...) const         &&, Index> : TFunctionParameterType<RetType(ParamTypes...), Index> {};
template <typename RetType, typename... ParamTypes, unsigned int Index> struct TFunctionParameterType<RetType(ParamTypes...)       volatile  , Index> : TFunctionParameterType<RetType(ParamTypes...), Index> {};
template <typename RetType, typename... ParamTypes, unsigned int Index> struct TFunctionParameterType<RetType(ParamTypes...)       volatile& , Index> : TFunctionParameterType<RetType(ParamTypes...), Index> {};
template <typename RetType, typename... ParamTypes, unsigned int Index> struct TFunctionParameterType<RetType(ParamTypes...)       volatile&&, Index> : TFunctionParameterType<RetType(ParamTypes...), Index> {};
template <typename RetType, typename... ParamTypes, unsigned int Index> struct TFunctionParameterType<RetType(ParamTypes...) const volatile  , Index> : TFunctionParameterType<RetType(ParamTypes...), Index> {};
template <typename RetType, typename... ParamTypes, unsigned int Index> struct TFunctionParameterType<RetType(ParamTypes...) const volatile& , Index> : TFunctionParameterType<RetType(ParamTypes...), Index> {};
template <typename RetType, typename... ParamTypes, unsigned int Index> struct TFunctionParameterType<RetType(ParamTypes...) const volatile&&, Index> : TFunctionParameterType<RetType(ParamTypes...), Index> {};

template <typename FunctionType, unsigned int Index>
using TFunctionParameterType_T = typename TFunctionParameterType<FunctionType, Index>::Type;
