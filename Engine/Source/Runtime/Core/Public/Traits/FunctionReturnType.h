// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Type trait which yields the return type of a function type, e.g.:
 *
 * TFunctionReturnType_T<int(float, bool, const char*)> == int
 *
 * This doesn't work for pointers or references to functions, or member function pointer types  - use the correct one of these to get a pure function type:
 * - std::remove_pointer_t
 * - std::remove_reference_t
 * - TRemoveMemberPtr_T
 */
template <typename FunctionType>
struct TFunctionReturnType
{
};

template <typename RetType, typename... ParamTypes> struct TFunctionReturnType<RetType(ParamTypes...)                 > { using Type = RetType; };
template <typename RetType, typename... ParamTypes> struct TFunctionReturnType<RetType(ParamTypes...)               & > { using Type = RetType; };
template <typename RetType, typename... ParamTypes> struct TFunctionReturnType<RetType(ParamTypes...)               &&> { using Type = RetType; };
template <typename RetType, typename... ParamTypes> struct TFunctionReturnType<RetType(ParamTypes...) const           > { using Type = RetType; };
template <typename RetType, typename... ParamTypes> struct TFunctionReturnType<RetType(ParamTypes...) const         & > { using Type = RetType; };
template <typename RetType, typename... ParamTypes> struct TFunctionReturnType<RetType(ParamTypes...) const         &&> { using Type = RetType; };
template <typename RetType, typename... ParamTypes> struct TFunctionReturnType<RetType(ParamTypes...)       volatile  > { using Type = RetType; };
template <typename RetType, typename... ParamTypes> struct TFunctionReturnType<RetType(ParamTypes...)       volatile& > { using Type = RetType; };
template <typename RetType, typename... ParamTypes> struct TFunctionReturnType<RetType(ParamTypes...)       volatile&&> { using Type = RetType; };
template <typename RetType, typename... ParamTypes> struct TFunctionReturnType<RetType(ParamTypes...) const volatile  > { using Type = RetType; };
template <typename RetType, typename... ParamTypes> struct TFunctionReturnType<RetType(ParamTypes...) const volatile& > { using Type = RetType; };
template <typename RetType, typename... ParamTypes> struct TFunctionReturnType<RetType(ParamTypes...) const volatile&&> { using Type = RetType; };

template <typename FunctionType>
using TFunctionReturnType_T = typename TFunctionReturnType<FunctionType>::Type;
