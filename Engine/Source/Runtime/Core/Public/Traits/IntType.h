// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

/**
 * Type trait which yields a signed integer type of a given number of bytes.
 * If there is no such type, the Type member type will be absent, allowing it to be used in SFINAE contexts.
 */
template <int NumBytes>
struct TSignedIntType
{
};

template <> struct TSignedIntType<1> { using Type = int8; };
template <> struct TSignedIntType<2> { using Type = int16; };
template <> struct TSignedIntType<4> { using Type = int32; };
template <> struct TSignedIntType<8> { using Type = int64; };

/**
 * Helper for TSignedIntType which expands out to the nested Type.
 */
template <int NumBytes>
using TSignedIntType_T = typename TSignedIntType<NumBytes>::Type;


/**
 * Type trait which yields an unsigned integer type of a given number of bytes.
 * If there is no such type, the Type member type will be absent, allowing it to be used in SFINAE contexts.
 */
template <int NumBytes>
struct TUnsignedIntType
{
};

template <> struct TUnsignedIntType<1> { using Type = uint8; };
template <> struct TUnsignedIntType<2> { using Type = uint16; };
template <> struct TUnsignedIntType<4> { using Type = uint32; };
template <> struct TUnsignedIntType<8> { using Type = uint64; };

/**
 * Helper for TUnsignedIntType which expands out to the nested Type.
 */
template <int NumBytes>
using TUnsignedIntType_T = typename TUnsignedIntType<NumBytes>::Type;
