// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "Traits/IsCharType.h"

#ifndef PLATFORM_TCHAR_IS_CHAR16
	#error "Traits/IsCharEncodingCompatibleWith.h should be included after platform headers"
#endif

namespace UE::Core::Private
{
	template <typename SrcEncoding, typename DestEncoding>
	constexpr bool IsCharEncodingCompatibleWithImpl()
	{
		if constexpr (!TIsCharType_V<SrcEncoding> || !TIsCharType_V<DestEncoding>)
		{
			return false;
		}
		else if constexpr (std::is_same_v<SrcEncoding, DestEncoding>)
		{
			return true;
		}
		else if constexpr (std::is_same_v<SrcEncoding, ANSICHAR> && std::is_same_v<DestEncoding, UTF8CHAR>)
		{
			return true;
		}
		else if constexpr (std::is_same_v<SrcEncoding, UCS2CHAR> && std::is_same_v<DestEncoding, UTF16CHAR>)
		{
			return true;
		}
		else if constexpr (std::is_same_v<SrcEncoding, WIDECHAR> && std::is_same_v<DestEncoding, UCS2CHAR>)
		{
			return true;
		}
		else if constexpr (std::is_same_v<SrcEncoding, UCS2CHAR> && std::is_same_v<DestEncoding, WIDECHAR>)
		{
			return true;
		}
#if PLATFORM_TCHAR_IS_CHAR16
		else if constexpr (std::is_same_v<SrcEncoding, WIDECHAR> && std::is_same_v<DestEncoding, wchar_t>)
		{
			return true;
		}
		else if constexpr (std::is_same_v<SrcEncoding, wchar_t> && std::is_same_v<DestEncoding, WIDECHAR>)
		{
			return true;
		}
#endif
		else
		{
			return false;
		}
	};
}

/**
 * Trait which tests if a source char type is binary compatible with a destination char type.

 * This is not commutative.  For example, ANSI is compatible with UTF-8, but UTF-8 is not compatible with ANSI.
 */
template <typename SrcEncoding, typename DestEncoding>
struct TIsCharEncodingCompatibleWith
{
	static constexpr bool Value = UE::Core::Private::IsCharEncodingCompatibleWithImpl<std::remove_cv_t<SrcEncoding>, std::remove_cv_t<DestEncoding>>();
};

template <typename SrcEncoding, typename DestEncoding>
constexpr inline bool TIsCharEncodingCompatibleWith_V = TIsCharEncodingCompatibleWith<SrcEncoding, DestEncoding>::Value;

template <typename SrcEncoding>
using TIsCharEncodingCompatibleWithTCHAR = TIsCharEncodingCompatibleWith<SrcEncoding, TCHAR>;

template <typename SrcEncoding>
using TIsCharEncodingCompatibleWithUTF8CHAR = TIsCharEncodingCompatibleWith<SrcEncoding, UTF8CHAR>;
