// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "Traits/IsFixedWidthCharEncoding.h"
#include "Traits/IsCharType.h"

namespace UE::Core::Private
{
	template <typename SrcEncoding, typename DestEncoding>
	constexpr bool IsCharEncodingSimplyConvertibleToImpl()
	{
		if constexpr (TIsCharEncodingCompatibleWith_V<SrcEncoding, DestEncoding>)
		{
			// Binary-compatible conversions are always simple
			return true;
		}
		else if constexpr (TIsFixedWidthCharEncoding_V<SrcEncoding> && sizeof(DestEncoding) >= sizeof(SrcEncoding))
		{
			// Converting from a fixed-width encoding to a wider or same encoding should always be possible,
			// as should ANSICHAR->UTF8CHAR and UCS2CHAR->UTF16CHAR
			return true;
		}
		else
		{
			return false;
		}
	}
}

/**
 * Trait which tests if code units of the source encoding are simply convertible (i.e. by assignment) to the destination encoding.
 */
template <typename SrcEncoding, typename DestEncoding>
struct TIsCharEncodingSimplyConvertibleTo
{
	static_assert(TIsCharType<SrcEncoding >::Value, "SrcEncoding is not a character encoding type");
	static_assert(TIsCharType<DestEncoding>::Value, "DestEncoding is not a character encoding type");

	static constexpr bool Value = UE::Core::Private::IsCharEncodingSimplyConvertibleToImpl<SrcEncoding, DestEncoding>();
};

template <typename SrcEncoding, typename DestEncoding>
constexpr inline bool TIsCharEncodingSimplyConvertibleTo_V = TIsCharEncodingSimplyConvertibleTo<SrcEncoding, DestEncoding>::Value;
