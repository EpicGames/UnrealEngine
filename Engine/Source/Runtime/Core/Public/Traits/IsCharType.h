// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Type trait which tests if a type is a character encoding type.
 */
template<typename T> struct TIsCharType            { enum { Value = false, value = false }; };
template<>           struct TIsCharType<ANSICHAR>  { enum { Value = true,  value = true  }; };
template<>           struct TIsCharType<UCS2CHAR>  { enum { Value = true,  value = true  }; };
#if !PLATFORM_UCS2CHAR_IS_UTF16CHAR
template<>           struct TIsCharType<UTF16CHAR> { enum { Value = true,  value = true  }; };
#endif
template<>           struct TIsCharType<WIDECHAR>  { enum { Value = true,  value = true  }; };
template<>           struct TIsCharType<UTF8CHAR>  { enum { Value = true,  value = true  }; };
template<>           struct TIsCharType<UTF32CHAR> { enum { Value = true,  value = true  }; };
#if PLATFORM_TCHAR_IS_CHAR16
template<>           struct TIsCharType<wchar_t>   { enum { Value = true,  value = true  }; };
#endif

template <typename T> struct TIsCharType<const          T> : TIsCharType<T> { };
template <typename T> struct TIsCharType<      volatile T> : TIsCharType<T> { };
template <typename T> struct TIsCharType<const volatile T> : TIsCharType<T> { };

template <typename T>
constexpr inline bool TIsCharType_V = TIsCharType<T>::Value;
