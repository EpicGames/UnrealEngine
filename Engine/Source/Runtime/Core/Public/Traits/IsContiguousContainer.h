// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/StaticAssertCompleteType.h"
#include <initializer_list>

/**
 * Traits class which tests if a type is a contiguous container.
 * Requires:
 *    [ &Container[0], &Container[0] + Num ) is a valid range
 */
template <typename T>
struct TIsContiguousContainer
{
	UE_STATIC_ASSERT_COMPLETE_TYPE(T, "TIsContiguousContainer instantiated with an incomplete type");
	// This sizeof(T) == 0 test will let Clang treat zero-sized arrays as contiguous containers.
	// An alternate fix using a specialization of TIsContiguousContainer (which Clang doesn't support) can be found below.
	static constexpr bool Value = sizeof(T) == 0;
};

template <typename T> struct TIsContiguousContainer<             T& > : TIsContiguousContainer<T> {};
template <typename T> struct TIsContiguousContainer<             T&&> : TIsContiguousContainer<T> {};
template <typename T> struct TIsContiguousContainer<const          T> : TIsContiguousContainer<T> {};
template <typename T> struct TIsContiguousContainer<      volatile T> : TIsContiguousContainer<T> {};
template <typename T> struct TIsContiguousContainer<const volatile T> : TIsContiguousContainer<T> {};

/**
 * Specialization for C arrays (always contiguous)
 */
template <typename T, size_t N> struct TIsContiguousContainer<               T[N]> { static constexpr bool Value = true; };
template <typename T, size_t N> struct TIsContiguousContainer<const          T[N]> { static constexpr bool Value = true; };
template <typename T, size_t N> struct TIsContiguousContainer<      volatile T[N]> { static constexpr bool Value = true; };
template <typename T, size_t N> struct TIsContiguousContainer<const volatile T[N]> { static constexpr bool Value = true; };

/**
 * Specialization for zero-sized arrays - which are not handled by the above Clang-specific fix in the primary TIsContiguousContainer template.
 */
#ifndef __clang__
	template <typename T> struct TIsContiguousContainer<               T[0]> { static constexpr bool Value = true; };
	template <typename T> struct TIsContiguousContainer<const          T[0]> { static constexpr bool Value = true; };
	template <typename T> struct TIsContiguousContainer<      volatile T[0]> { static constexpr bool Value = true; };
	template <typename T> struct TIsContiguousContainer<const volatile T[0]> { static constexpr bool Value = true; };
#endif

/**
 * Specialization for unbounded C arrays (never contiguous - should be treated as pointers which are not regarded as contiguous containers)
 */
template <typename T> struct TIsContiguousContainer<               T[]> { static constexpr bool Value = false; };
template <typename T> struct TIsContiguousContainer<const          T[]> { static constexpr bool Value = false; };
template <typename T> struct TIsContiguousContainer<      volatile T[]> { static constexpr bool Value = false; };
template <typename T> struct TIsContiguousContainer<const volatile T[]> { static constexpr bool Value = false; };

/**
 * Specialization for initializer lists (also always contiguous)
 */
template <typename T>
struct TIsContiguousContainer<std::initializer_list<T>>
{
	static constexpr bool Value = true;
};

template <typename T>
inline constexpr bool TIsContiguousContainer_V = TIsContiguousContainer<T>::Value;
