// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "Traits/IsCharType.h"

#ifndef PLATFORM_TCHAR_IS_CHAR16
	#error "Traits/IsFixedWidthCharEncoding.h should be included after platform headers"
#endif

namespace UE::Core::Private
{
	/**
	 * Tests whether an encoding has fixed-width characters
	 */
	template <typename Encoding>
	constexpr bool IsFixedWidthEncodingImpl()
	{
		return
			std::is_same_v<Encoding, ANSICHAR> ||
			std::is_same_v<Encoding, UCS2CHAR> || // this may not be true when PLATFORM_UCS2CHAR_IS_UTF16CHAR == 1, but this is the legacy behavior
			std::is_same_v<Encoding, WIDECHAR> || // the UCS2CHAR comment also applies to WIDECHAR
#if PLATFORM_TCHAR_IS_CHAR16
			std::is_same_v<Encoding, wchar_t> || // the UCS2CHAR comment also applies to wchar_t
#endif
			std::is_same_v<Encoding, UTF32CHAR>;
	}
}

template <typename Encoding>
struct TIsFixedWidthCharEncoding
{
	static_assert(TIsCharType_V<Encoding>, "Encoding is not a character encoding type");

	static constexpr bool Value = UE::Core::Private::IsFixedWidthEncodingImpl<std::remove_cv_t<Encoding>>();
};

template <typename Encoding>
constexpr inline bool TIsFixedWidthCharEncoding_V = TIsFixedWidthCharEncoding<Encoding>::Value;
