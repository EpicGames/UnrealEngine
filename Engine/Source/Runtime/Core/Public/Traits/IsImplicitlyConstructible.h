// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE::Core::Private
{
	template <typename T, typename... ArgTypes>
	struct TIsImplicitlyConstructibleImpl
	{
		template <typename U>
		static void TakesU(U&&);

		static constexpr bool value = (!std::is_aggregate_v<T> || sizeof...(ArgTypes) == 0) && requires{ TakesU<T>({ std::declval<ArgTypes>()... }); };
	};
	template <typename T, typename ArgType>
	struct TIsImplicitlyConstructibleImpl<T, ArgType>
	{
		static constexpr bool value = std::is_convertible_v<ArgType, T>;
		static constexpr bool Value = value;
	};
}

template <typename T, typename... ArgTypes>
struct TIsImplicitlyConstructible : UE::Core::Private::TIsImplicitlyConstructibleImpl<T, ArgTypes...>
{
};

template <typename T, typename... ArgTypes>
inline constexpr bool TIsImplicitlyConstructible_V = UE::Core::Private::TIsImplicitlyConstructibleImpl<T, ArgTypes...>::value;
