// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Traits classes and constants which test if a type is a TEnumAsByte.
 */

template <class InEnumType>
class TEnumAsByte;

template <typename T> inline constexpr bool TIsTEnumAsByte_V                                = false;
template <typename T> inline constexpr bool TIsTEnumAsByte_V<               TEnumAsByte<T>> = true;
template <typename T> inline constexpr bool TIsTEnumAsByte_V<const          TEnumAsByte<T>> = true;
template <typename T> inline constexpr bool TIsTEnumAsByte_V<      volatile TEnumAsByte<T>> = true;
template <typename T> inline constexpr bool TIsTEnumAsByte_V<const volatile TEnumAsByte<T>> = true;

template <typename T>
struct TIsTEnumAsByte
{
	static constexpr bool Value = TIsTEnumAsByte_V<T>;
	static constexpr bool value = Value;
};
