// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"

/**
 * Traits classes and constants which test if a type is a UE string class.
 */

template <typename T> inline constexpr bool TIsTString_V              = false;
template <>           inline constexpr bool TIsTString_V<FAnsiString> = true;
template <>           inline constexpr bool TIsTString_V<FUtf8String> = true;
template <>           inline constexpr bool TIsTString_V<FWideString> = true;

template <typename T> inline constexpr bool TIsTString_V<const          T> = TIsTString_V<T>;
template <typename T> inline constexpr bool TIsTString_V<      volatile T> = TIsTString_V<T>;
template <typename T> inline constexpr bool TIsTString_V<const volatile T> = TIsTString_V<T>;

template <typename T>
struct TIsTString
{
	static constexpr bool Value = TIsTString_V<T>;
	static constexpr bool value = Value;
};
