// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

template <typename T>
class TSubclassOf;

template <typename T>
inline constexpr bool TIsTSubclassOf_V = false;

template <typename T> inline constexpr bool TIsTSubclassOf_V<               TSubclassOf<T>> = true;
template <typename T> inline constexpr bool TIsTSubclassOf_V<const          TSubclassOf<T>> = true;
template <typename T> inline constexpr bool TIsTSubclassOf_V<      volatile TSubclassOf<T>> = true;
template <typename T> inline constexpr bool TIsTSubclassOf_V<const volatile TSubclassOf<T>> = true;

template <typename T>
struct TIsTSubclassOf
{
	static constexpr bool Value = TIsTSubclassOf_V<T>;
};
