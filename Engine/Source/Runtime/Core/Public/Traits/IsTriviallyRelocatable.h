// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

/**
 * A trait which tests if a type is trivially relocatable.
 *
 * NOTE: In UE, all types are assumed to be trivially relocatable, so this defaults to true. The intent of this
 *       trait is for types to specialize it as false when it is known that they are not trivially relocatable,
 *       and for containers and other utility functions which use trivial relocation to to inspect this trait and
 *       guard against use with types which are known to be not trivially relocatable.
 */
template <typename T>
struct TIsTriviallyRelocatable
{
	// We allow bitwise relocation of types containing e.g. const and reference members, but we don't want to allow 
	// relocation of types which are UE_NONCOPYABLE or equivalent.
	//
	// See note above as to why this is why it is. In C++26, we should be able to catch more
	// by using this: https://www.open-std.org/jtc1/sc22/wg21/docs/papers/2024/p2786r4.pdf
	//
	// Should be the test below by default, but there are a lot of existing violations that will need to be fixed first.

//	static inline constexpr bool Value = std::is_move_constructible_v<T>;
	static inline constexpr bool Value = true;
};

template <typename T> struct TIsTriviallyRelocatable<const          T> : TIsTriviallyRelocatable<T> {};
template <typename T> struct TIsTriviallyRelocatable<      volatile T> : TIsTriviallyRelocatable<T> {};
template <typename T> struct TIsTriviallyRelocatable<const volatile T> : TIsTriviallyRelocatable<T> {};

template <typename T>
inline constexpr bool TIsTriviallyRelocatable_V = TIsTriviallyRelocatable<T>::Value;
