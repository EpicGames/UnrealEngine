// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.8, "MemberFunctionPtrOuter.h has been deprecated.")
#include "Traits/MemberFunctionPtrOuterDeprecated.h"
