// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

// THIS HEADER IS DEPRECATD and shouldn't be included directly.
//
// It is only intended to be included by other deprecated contexts.
// When those deprecated contexts are removed, this header can be removed too.

/**
 * Type trait which yields the type of the class given a pointer to a member function of that class, e.g.:
 *
 * TMemberFunctionPtrOuter_T<decltype(&FVector::Dot)> == FVector
 */
template <typename T>
struct UE_DEPRECATED(5.8, "TMemberFunctionPtrOuter has been deprecated, please use TMemberPtrOuter instead.") TMemberFunctionPtrOuter;

template <typename ReturnType, typename ObjectType, typename... ArgTypes> struct UE_DEPRECATED(5.8, "TMemberFunctionPtrOuter has been deprecated, use TMemberPtrOuter instead.") TMemberFunctionPtrOuter<ReturnType (ObjectType::*)(ArgTypes...)                 > { using Type = ObjectType; };
template <typename ReturnType, typename ObjectType, typename... ArgTypes> struct UE_DEPRECATED(5.8, "TMemberFunctionPtrOuter has been deprecated, use TMemberPtrOuter instead.") TMemberFunctionPtrOuter<ReturnType (ObjectType::*)(ArgTypes...)               & > { using Type = ObjectType; };
template <typename ReturnType, typename ObjectType, typename... ArgTypes> struct UE_DEPRECATED(5.8, "TMemberFunctionPtrOuter has been deprecated, use TMemberPtrOuter instead.") TMemberFunctionPtrOuter<ReturnType (ObjectType::*)(ArgTypes...)               &&> { using Type = ObjectType; };
template <typename ReturnType, typename ObjectType, typename... ArgTypes> struct UE_DEPRECATED(5.8, "TMemberFunctionPtrOuter has been deprecated, use TMemberPtrOuter instead.") TMemberFunctionPtrOuter<ReturnType (ObjectType::*)(ArgTypes...) const           > { using Type = ObjectType; };
template <typename ReturnType, typename ObjectType, typename... ArgTypes> struct UE_DEPRECATED(5.8, "TMemberFunctionPtrOuter has been deprecated, use TMemberPtrOuter instead.") TMemberFunctionPtrOuter<ReturnType (ObjectType::*)(ArgTypes...) const         & > { using Type = ObjectType; };
template <typename ReturnType, typename ObjectType, typename... ArgTypes> struct UE_DEPRECATED(5.8, "TMemberFunctionPtrOuter has been deprecated, use TMemberPtrOuter instead.") TMemberFunctionPtrOuter<ReturnType (ObjectType::*)(ArgTypes...) const         &&> { using Type = ObjectType; };
template <typename ReturnType, typename ObjectType, typename... ArgTypes> struct UE_DEPRECATED(5.8, "TMemberFunctionPtrOuter has been deprecated, use TMemberPtrOuter instead.") TMemberFunctionPtrOuter<ReturnType (ObjectType::*)(ArgTypes...)       volatile  > { using Type = ObjectType; };
template <typename ReturnType, typename ObjectType, typename... ArgTypes> struct UE_DEPRECATED(5.8, "TMemberFunctionPtrOuter has been deprecated, use TMemberPtrOuter instead.") TMemberFunctionPtrOuter<ReturnType (ObjectType::*)(ArgTypes...)       volatile& > { using Type = ObjectType; };
template <typename ReturnType, typename ObjectType, typename... ArgTypes> struct UE_DEPRECATED(5.8, "TMemberFunctionPtrOuter has been deprecated, use TMemberPtrOuter instead.") TMemberFunctionPtrOuter<ReturnType (ObjectType::*)(ArgTypes...)       volatile&&> { using Type = ObjectType; };
template <typename ReturnType, typename ObjectType, typename... ArgTypes> struct UE_DEPRECATED(5.8, "TMemberFunctionPtrOuter has been deprecated, use TMemberPtrOuter instead.") TMemberFunctionPtrOuter<ReturnType (ObjectType::*)(ArgTypes...) const volatile  > { using Type = ObjectType; };
template <typename ReturnType, typename ObjectType, typename... ArgTypes> struct UE_DEPRECATED(5.8, "TMemberFunctionPtrOuter has been deprecated, use TMemberPtrOuter instead.") TMemberFunctionPtrOuter<ReturnType (ObjectType::*)(ArgTypes...) const volatile& > { using Type = ObjectType; };
template <typename ReturnType, typename ObjectType, typename... ArgTypes> struct UE_DEPRECATED(5.8, "TMemberFunctionPtrOuter has been deprecated, use TMemberPtrOuter instead.") TMemberFunctionPtrOuter<ReturnType (ObjectType::*)(ArgTypes...) const volatile&&> { using Type = ObjectType; };

PRAGMA_DISABLE_DEPRECATION_WARNINGS
template <typename T>
using TMemberFunctionPtrOuter_T UE_DEPRECATED(5.8, "TMemberFunctionPtrOuter_T has been deprecated, use TMemberPtrOuter_T instead.") = typename TMemberFunctionPtrOuter<T>::Type;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
