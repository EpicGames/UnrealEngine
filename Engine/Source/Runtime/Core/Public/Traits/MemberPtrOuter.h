// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Type trait which yields the type of the class given a pointer to a member of that class, e.g.:
 *
 * TMemberPtrOuter_T<decltype(&FVector::X)>   == FVector
 * TMemberPtrOuter_T<decltype(&FVector::Dot)> == FVector
 */
template <typename T>
struct TMemberPtrOuter
{
};

template <typename OuterType, typename MemberType>
struct TMemberPtrOuter<MemberType OuterType::*>
{
	using Type = OuterType;
};

template <typename T>
using TMemberPtrOuter_T = typename TMemberPtrOuter<T>::Type;
