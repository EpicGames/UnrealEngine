// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Type trait which yields the type being pointed to of a member pointer type, e.g.:
 *
 * TRemoveMemberPtr_T<decltype(&FVector::X)>   == double
 * TRemoveMemberPtr_T<decltype(&FVector::Dot)> == double(const FVector&)
 */
template <typename T>
struct TRemoveMemberPtr
{
	using Type = T;
};

template <typename OuterType, typename MemberType>
struct TRemoveMemberPtr<MemberType OuterType::*>
{
	using Type = MemberType;
};

template <typename T>
using TRemoveMemberPtr_T = typename TRemoveMemberPtr<T>::Type;
