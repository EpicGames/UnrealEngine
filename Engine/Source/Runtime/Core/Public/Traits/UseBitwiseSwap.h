// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include <type_traits>

/**
 * A traits class which specifies whether a relocation of a given type should use a bitwise function like memcpy or memswap or traditional value-based operations.
 */
template <typename T>
struct TUseBitwiseSwap
{
	// We don't use bitwise operations for 'register' types because this will force them into memory and be slower.
	enum { Value = !(std::is_enum_v<T> || std::is_pointer_v<T> || std::is_arithmetic_v<T>) };
};
