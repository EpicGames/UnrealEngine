// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes made in Dev-Anim stream
struct FAnimObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded,

		// Reworked how anim blueprint root nodes are recovered
		LinkTimeAnimBlueprintRootDiscovery,

		// Cached marker sync names on skeleton for editor
		StoreMarkerNamesOnSkeleton,

		// Serialized register array state for RigVM
		SerializeRigVMRegisterArrayState,

		// Increase number of bones per chunk from uint8 to uint16
		IncreaseBoneIndexLimitPerChunk,

		UnlimitedBoneInfluences,

		// Anim sequences have colors for their curves
		AnimSequenceCurveColors,

		// Notifies and sync markers now have Guids
		NotifyAndSyncMarkerGuids,

		// Serialized register dynamic state for RigVM
		SerializeRigVMRegisterDynamicState,

		// Groom cards serialization
		SerializeGroomCards,

		// Serialized rigvm entry names
		SerializeRigVMEntries,

		// Serialized rigvm entry names
		SerializeHairBindingAsset,

		// Serialized rigvm entry names
		SerializeHairClusterCullingData,

		// Groom cards and meshes serialization
		SerializeGroomCardsAndMeshes,

		// Stripping LOD data from groom
		GroomLODStripping,

		// Stripping LOD data from groom
		GroomBindingSerialization,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

private:
	FAnimObjectVersion() {}
};
