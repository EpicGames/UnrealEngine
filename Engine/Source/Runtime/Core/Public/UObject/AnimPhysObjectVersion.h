// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes made in Dev-AnimPhys stream
struct FAnimPhysObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded,
		// convert animnode look at to use just default axis instead of enum, which doesn't do much
		ConvertAnimNodeLookAtAxis,
		// Change FKSphylElem and FKBoxElem to use Rotators not Quats for easier editing
		BoxSphylElemsUseRotators,
		// Change thumbnail scene info and asset import data to be transactional
		ThumbnailSceneInfoAndAssetImportDataAreTransactional,
		// Enabled clothing masks rather than painting parameters directly
		AddedClothingMaskWorkflow,
		// Remove UID from smart name serialize, it just breaks determinism 
		RemoveUIDFromSmartNameSerialize,
		// Convert FName Socket to FSocketReference and added TargetReference that support bone and socket
		CreateTargetReference,
		// Tune soft limit stiffness and damping coefficients
		TuneSoftLimitStiffnessAndDamping,
		// Fix possible inf/nans in clothing particle masses
		FixInvalidClothParticleMasses,
		// Moved influence count to cached data
		CacheClothMeshInfluences,
		// Remove GUID from Smart Names entirely + remove automatic name fixup
		SmartNameRefactorForDeterministicCooking,
		// rename the variable and allow individual curves to be set
		RenameDisableAnimCurvesToAllowAnimCurveEvaluation,
		// link curve to LOD, so curve metadata has to include LODIndex
		AddLODToCurveMetaData,
		// Fixed blend profile references persisting after paste when they aren't compatible
		FixupBadBlendProfileReferences,
		// Allowing multiple audio plugin settings
		AllowMultipleAudioPluginSettings,
		// Change RetargetSource reference to SoftObjectPtr
		ChangeRetargetSourceReferenceToSoftObjectPtr,
		// Save editor only full pose for pose asset 
		SaveEditorOnlyFullPoseForPoseAsset,
		// Asset change and cleanup to facilitate new streaming system
		GeometryCacheAssetDeprecation,
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

private:
	FAnimPhysObjectVersion() {}
};
