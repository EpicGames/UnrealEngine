// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes made in Dev-Blueprints stream
struct FBlueprintsObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,
		OverridenEventReferenceFixup,
		CleanBlueprintFunctionFlags,
		ArrayGetByRefUpgrade,
		EdGraphPinOptimized,
		AllowDeletionConformed,
		AdvancedContainerSupport,
		SCSHasComponentTemplateClass,
		ComponentTemplateClassSupport,
		ArrayGetFuncsReplacedByCustomNode,
		DisallowObjectConfigVars,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

private:
	FBlueprintsObjectVersion() {}
};
