// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UEnum;

/*-----------------------------------------------------------------------------
C++ templated Static(Enum) retrieval function prototypes.
-----------------------------------------------------------------------------*/

template <typename EnumType> UEnum* StaticEnum() = delete;
