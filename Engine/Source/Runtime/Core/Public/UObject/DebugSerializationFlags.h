// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
DebugSerializationFlags.h: 
Set custom flags on the archive to help track issues while serializing
=============================================================================*/

#pragma once

#include "CoreTypes.h"

// Debug serialization flags
enum EDebugSerializationFlags
{
	/** No special flags */
	DSF_None =					0x00000000,

	/**
	 * If FDiffSerializeArchive is being used, instruct it to NOT report diffs while this flag is set.
	 * This is used e.g. when serializing offsets that are likely to change when there is any other change
	 * in the serialization of the package.
	 */
	DSF_IgnoreDiff UE_DEPRECATED(5.0, "Diffing now compares the final value after serialization is complete; marking diffs is usually not required. Use FArchiveStackTraceIgnoreScope where it is still necessary.")
		= 0x00000001,
	DSF_EnableCookerWarnings =	0x00000002,
};
