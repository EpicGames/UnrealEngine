// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Concepts/UEnum.h"
#include "Concepts/UScriptStruct.h"
#include "Traits/IsTEnumAsByte.h"
#include "UObject/NameTypes.h"
#include "UObject/NativeTypeToPropertyType.h"
#include "UObject/PropertyTypeToFName.h"

#include <type_traits>

class UObjectBase;
class UScriptStruct;

template <typename T>
struct TBaseStructure;

// Describes a single field in a delegate payload schema.  This is a CoreUObject-neutral
// representation that can be constructed in Core from compile-time types and later interpreted
// by CoreUObject to build FProperty instances and UScriptStructs.
struct FDelegatePayloadFieldDesc
{
	// The name of the parameter this field corresponds to in the delegate's UFunction signature.
	FName FieldName;

	// The FProperty class name, e.g. NAME_IntProperty, NAME_StructProperty.
	// Uses the pre-registered EName constants from UnrealNames.inl.
	FName PropertyTypeName;

	// Supplementary type identifier for types that need it:
	// - StructProperty: struct path name (e.g. "Vector")
	// - EnumProperty / ByteProperty with enum: enum name
	// - Object/Class/SoftObject/etc: class name
	// NAME_None for simple numeric/string types.
	FName ExtendedTypeName;

	// Child descriptors for container inner types:
	// - ArrayProperty / SetProperty / OptionalProperty: 1 inner field (element type)
	// - MapProperty: 2 inner fields (key type, value type)
	// Empty for non-container types.
	TArray<FDelegatePayloadFieldDesc> InnerFields;
};

// Describes the complete field layout of a delegate payload.
struct FDelegatePayloadSchema
{
	TArray<FDelegatePayloadFieldDesc> Fields;
};


// Recursively builds a FDelegatePayloadFieldDesc for a native type, including container inner types.
template <typename T>
FDelegatePayloadFieldDesc MakePayloadFieldDesc(FName FieldName)
{
	using UnqualifiedT = std::remove_cv_t<T>;
	using PropType = TNativeTypeToPropertyType_T<UnqualifiedT>;

	FDelegatePayloadFieldDesc Desc;
	Desc.FieldName = FieldName;
	Desc.PropertyTypeName = TPropertyTypeToFName<PropType>::Get();

	// Set ExtendedTypeName for types that need supplementary identification
	if constexpr (UE::CUScriptStruct<UnqualifiedT>)
	{
		Desc.ExtendedTypeName = TBaseStructure<UnqualifiedT>::Get()->GetFName();
	}
	else if constexpr (TIsTEnumAsByte_V<UnqualifiedT>)
	{
		Desc.ExtendedTypeName = StaticEnum<typename UnqualifiedT::EnumType>()->GetFName();
	}
	else if constexpr (UE::CUEnum<UnqualifiedT>)
	{
		Desc.ExtendedTypeName = StaticEnum<UnqualifiedT>()->GetFName();
	}

	// Recurse into container inner types
	if constexpr (std::is_same_v<PropType, FArrayProperty>)
	{
		Desc.InnerFields = { MakePayloadFieldDesc<typename T::ElementType>(NAME_None) };
	}
	else if constexpr (std::is_same_v<PropType, FSetProperty>)
	{
		Desc.InnerFields = { MakePayloadFieldDesc<typename T::ElementType>(NAME_None) };
	}
	else if constexpr (std::is_same_v<PropType, FOptionalProperty>)
	{
		Desc.InnerFields = { MakePayloadFieldDesc<typename T::ElementType>(NAME_None) };
	}
	else if constexpr (std::is_same_v<PropType, FMapProperty>)
	{
		Desc.InnerFields = { MakePayloadFieldDesc<typename T::KeyType>(NAME_None), MakePayloadFieldDesc<typename T::ValueType>(NAME_None) };
	}

	return Desc;
}
