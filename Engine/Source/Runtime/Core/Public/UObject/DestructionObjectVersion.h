// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes made in Dev-Destruction stream
struct FDestructionObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// Added timestamped caches for geometry component to handle transform sampling instead of per-frame
		AddedTimestampedGeometryComponentCache,

		// Added functionality to strip unnecessary data from geometry collection caches
		AddedCacheDataReduction,

		// Geometry collection data is now in the DDC
		GeometryCollectionInDDC,

		// Geometry collection data is now in both the DDC and the asset
		GeometryCollectionInDDCAndAsset,

		// New way to serialize unique ptr and serializable ptr
		ChaosArchiveAdded,

		// Serialization support for UFieldSystems
		FieldsAdded,

		// density default units changed from kg/cm3 to kg/m3
		DensityUnitsChanged,

		// bulk serialize arrays
		BulkSerializeArrays,

		// bulk serialize arrays
		GroupAndAttributeNameRemapping,

		// bulk serialize arrays
		ImplicitObjectDoCollideAttribute,


		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

private:
	FDestructionObjectVersion() {}
};
