// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "UObject/NameTypes.h"
#include "Misc/Guid.h"
#include "Serialization/CustomVersion.h"

struct FDevSystemGuidRegistration
{
public:
	CORE_API FDevSystemGuidRegistration(const TMap<FGuid, FGuid>& SystemGuids);
};

struct FDevSystemGuids
{
	static CORE_API const FDevSystemGuids& Get();
	static CORE_API FGuid GetSystemGuid(FGuid System);
	
	CORE_API FDevSystemGuids();
	const FGuid GLOBALSHADERMAP_DERIVEDDATA_VER;
	const FGuid GROOM_BINDING_DERIVED_DATA_VERSION;
	const FGuid GROOM_DERIVED_DATA_VERSION;
	const FGuid LANDSCAPE_MOBILE_COOK_VERSION;
	const FGuid MATERIALSHADERMAP_DERIVEDDATA_VER;
	const FGuid NANITE_DERIVEDDATA_VER;
	const FGuid NIAGARASHADERMAP_DERIVEDDATA_VER;
	const FGuid Niagara_LatestScriptCompileVersion;
	const FGuid POSESEARCHDB_DERIVEDDATA_VER;
	const FGuid SkeletalMeshDerivedDataVersion;
	const FGuid STATICMESH_DERIVEDDATA_VER;
	const FGuid MaterialTranslationDDCVersion;
};

class FDevVersionRegistration :  public FCustomVersionRegistration
{
public:
	/** @param InFriendlyName must be a string literal */
	template<int N>
	FDevVersionRegistration(FGuid InKey, int32 Version, const TCHAR(&InFriendlyName)[N], CustomVersionValidatorFunc InValidatorFunc = nullptr)
		: FCustomVersionRegistration(InKey, Version, InFriendlyName, InValidatorFunc)
	{
		RecordDevVersion(InKey);
	}

	/** Dumps all registered versions to log */
	static CORE_API void DumpVersionsToLog();
private:
	static CORE_API void RecordDevVersion(FGuid Key);
};
