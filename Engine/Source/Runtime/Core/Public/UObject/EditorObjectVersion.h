// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes made in Dev-Editor stream
struct FEditorObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,
		// Localizable text gathered and stored in packages is now flagged with a localizable text gathering process version
		GatheredTextProcessVersionFlagging,
		// Fixed several issues with the gathered text cache stored in package headers
		GatheredTextPackageCacheFixesV1,
		// Added support for "root" meta-data (meta-data not associated with a particular object in a package)
		RootMetaDataSupport,
		// Fixed issues with how Blueprint bytecode was cached
		GatheredTextPackageCacheFixesV2,
		// Updated FFormatArgumentData to allow variant data to be marshaled from a BP into C++
		TextFormatArgumentDataIsVariant,
		// Changes to SplineComponent
		SplineComponentCurvesInStruct,
		// Updated ComboBox to support toggling the menu open, better controller support
		ComboBoxControllerSupportUpdate,
		// Refactor mesh editor materials
		RefactorMeshEditorMaterials,
		// Added UFontFace assets
		AddedFontFaceAssets,
		// Add UPROPERTY for TMap of Mesh section, so the serialize will be done normally (and export to text will work correctly)
		UPropertryForMeshSection,
		// Update the schema of all widget blueprints to use the WidgetGraphSchema
		WidgetGraphSchema,
		// Added a specialized content slot to the background blur widget
		AddedBackgroundBlurContentSlot,
		// Updated UserDefinedEnums to have stable keyed display names
		StableUserDefinedEnumDisplayNames,
		// Added "Inline" option to UFontFace assets
		AddedInlineFontFaceAssets,
		// Fix a serialization issue with static mesh FMeshSectionInfoMap FProperty
		UPropertryForMeshSectionSerialize,
		// Adding a version bump for the new fast widget construction in case of problems.
		FastWidgetTemplates,
		// Update material thumbnails to be more intelligent on default primitive shape for certain material types
		MaterialThumbnailRenderingChanges,
		// Introducing a new clipping system for Slate/UMG
		NewSlateClippingSystem,
		// MovieScene Meta Data added as native Serialization
		MovieSceneMetaDataSerialization,
		// Text gathered from properties now adds two variants: a version without the package localization ID (for use at runtime), and a version with it (which is editor-only)
		GatheredTextEditorOnlyPackageLocId,
		// Added AlwaysSign to FNumberFormattingOptions
		AddedAlwaysSignNumberFormattingOption,
		// Added additional objects that must be serialized as part of this new material feature
		AddedMaterialSharedInputs,
		// Added morph target section indices
		AddedMorphTargetSectionIndices,
		// Serialize the instanced static mesh render data, to avoid building it at runtime
		SerializeInstancedStaticMeshRenderData,
		// Change to MeshDescription serialization (moved to release)
		MeshDescriptionNewSerialization_MovedToRelease,
		// New format for mesh description attributes
		MeshDescriptionNewAttributeFormat,
		// Switch root component of SceneCapture actors from MeshComponent to SceneComponent
		ChangeSceneCaptureRootComponent,
		// StaticMesh serializes MeshDescription instead of RawMesh
		StaticMeshDeprecatedRawMesh,
		// MeshDescriptionBulkData contains a Guid used as a DDC key
		MeshDescriptionBulkDataGuid,
		// Change to MeshDescription serialization (removed FMeshPolygon::HoleContours)
		MeshDescriptionRemovedHoles,
		// Change to the WidgetCompoent WindowVisibilty default value
		ChangedWidgetComponentWindowVisibilityDefault,
		// Avoid keying culture invariant display strings during serialization to avoid non-deterministic cooking issues
		CultureInvariantTextSerializationKeyStability,
		// Change to UScrollBar and UScrollBox thickness property (removed implicit padding of 2, so thickness value must be incremented by 4).
		ScrollBarThicknessChange,
		// Deprecated LandscapeHoleMaterial
		RemoveLandscapeHoleMaterial,
		// MeshDescription defined by triangles instead of arbitrary polygons
		MeshDescriptionTriangles,
		//Add weighted area and angle when computing the normals
		ComputeWeightedNormals,
		// SkeletalMesh now can be rebuild in editor, no more need to re-import
		SkeletalMeshBuildRefactor,
		// Move all SkeletalMesh source data into a private uasset in the same package has the skeletalmesh
		SkeletalMeshMoveEditorSourceDataToPrivateAsset,
		// Parse text only if the number is inside the limits of its type
		NumberParsingOptionsNumberLimitsAndClamping,
		//Make sure we can have more then 255 material in the skeletal mesh source data
		SkeletalMeshSourceDataSupport16bitOfMaterialNumber,
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

private:
	FEditorObjectVersion() {}
};
