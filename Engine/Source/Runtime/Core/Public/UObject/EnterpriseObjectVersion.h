// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes made in Dev-Enterprise stream
struct FEnterpriseObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// Conditional serialization of bulk (UDatasmithScene) and extra (UDatasmithStaticMeshCADImportData) data 
		FixSerializationOfBulkAndExtraData,

		// Extensibility updates for bookmarks
		BookmarkExtensibilityUpgrade,

		// Update FMediaFrameworkCaptureCameraViewportCameraOutputInfo with LazyObjectPtr
		MediaFrameworkUserDataLazyObject,
		
		// Live Live timecode synchronization updates
		LiveLinkTimeSynchronization,

		// Deprecate MediaMode and MediaPort for from MediaConfiguration
		AjaMediaConfiguration,

		// Addition of pointer to UDatasmithDataPrepRecipe 
		HasUDataprepRecipe,

		// Addition of UDatatprepAssetInterface and UDatatprepAssetInstance
		HasUDatatprepAssetInterface,
		
		// MeshDescriptionBulkData contains a bGuidIsHash so we can benefit from DDC caching.
		MeshDescriptionBulkDataGuidIsHash,
		
		// Splitting controller associated with a livelink component for each LiveLinkRole class hierarchy
		LiveLinkControllerSplitPerRole,

		// Change CoreTechParametricSurface serialization to be more efficient
		CoreTechParametricSurfaceOptim,

		// Replacement of UDatasmithParametricSurfaceData by UParametricSurfaceData
		AddedParametricSurfaceData,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

	FEnterpriseObjectVersion() = delete;
};
