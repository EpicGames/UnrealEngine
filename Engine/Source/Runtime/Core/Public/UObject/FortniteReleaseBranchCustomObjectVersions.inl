// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(DEFINE_FORTNITE_RELEASE_VERSIONS)
#define FORTNITE_RELEASE_VERSION(Version, ID) Version,
#elif defined(CHECK_FORTNITE_RELEASE_VERSIONS)
#define FORTNITE_RELEASE_VERSION(Version, ID) static_assert(FFortniteReleaseBranchCustomObjectVersion::Version == ID);
#endif

// Before any version changes were made
FORTNITE_RELEASE_VERSION(BeforeCustomVersionWasAdded, 0)

// Custom 14.10 File Object Version
FORTNITE_RELEASE_VERSION(DisableLevelset_v14_10, 1)

// Add the long range attachment tethers to the cloth asset to avoid a large hitch during the cloth's initialization.
FORTNITE_RELEASE_VERSION(ChaosClothAddTethersToCachedData, 2)

// Chaos::TKinematicTarget no longer stores a full transform, only position/rotation.
FORTNITE_RELEASE_VERSION(ChaosKinematicTargetRemoveScale, 3)

// Move UCSModifiedProperties out of ActorComponent and in to sparse storage
FORTNITE_RELEASE_VERSION(ActorComponentUCSModifiedPropertiesSparseStorage, 4)

// Fixup Nanite meshes which were using the wrong material and didn't have proper UVs :
FORTNITE_RELEASE_VERSION(FixupNaniteLandscapeMeshes, 5)

// Remove any cooked collision data from nanite landscape / editor spline meshes since collisions are not needed there :
FORTNITE_RELEASE_VERSION(RemoveUselessLandscapeMeshesCookedCollisionData, 6)

// Serialize out UAnimCurveCompressionCodec::InstanceGUID to maintain deterministic DDC key generation in cooked-editor
FORTNITE_RELEASE_VERSION(SerializeAnimCurveCompressionCodecGuidOnCook, 7)

// Fix the Nanite landscape mesh being reused because of a bad name
FORTNITE_RELEASE_VERSION(FixNaniteLandscapeMeshNames, 8)

// Fixup and synchronize shared properties modified before the synchronicity enforcement
FORTNITE_RELEASE_VERSION(LandscapeSharedPropertiesEnforcement, 9)

// Include the cell size when computing the cell guid
FORTNITE_RELEASE_VERSION(WorldPartitionRuntimeCellGuidWithCellSize, 10)

// Enable SkipOnlyEditorOnly style cooking of NaniteOverrideMaterial
FORTNITE_RELEASE_VERSION(NaniteMaterialOverrideUsesEditorOnly, 11)

// Store game thread particles data in single precision
FORTNITE_RELEASE_VERSION(SinglePrecisonParticleData, 12)

// UPCGPoint custom serialization
FORTNITE_RELEASE_VERSION(PCGPointStructuredSerializer, 13)

// Deprecation of Nav Movement Properties and moving them to a new struct
FORTNITE_RELEASE_VERSION(NavMovementComponentMovingPropertiesToStruct, 14)

// Add bone serialization for dynamic mesh attributes
FORTNITE_RELEASE_VERSION(DynamicMeshAttributesSerializeBones, 15)

// Add option for sanitizing output attribute names for all PCG data getters
FORTNITE_RELEASE_VERSION(OptionSanitizeOutputAttributeNamesPCG, 16)

// Add automatic platform naming fix up for CommonUI input action data tables
FORTNITE_RELEASE_VERSION(CommonUIPlatformNamingUpgradeOption, 17)

// Add new fictitious forces modes to the cloth simulation
FORTNITE_RELEASE_VERSION(ChaosClothAddNewFictitiousForcesModes, 18)

// Updated how blend spaces calculate the effective animation length of the sample point
FORTNITE_RELEASE_VERSION(BlendSpaceUpdatedSamplePointAnimationLengthDerivation, 19)

// Fix to make the override option to "Use Half Precision" work in Material Editor when the cvar r.Mobile.FloatPrecisionMode=2
FORTNITE_RELEASE_VERSION(MaterialHalfPrecisionFixup, 20)

// -----<new versions can be added above this line>-------------------------------------------------
#undef FORTNITE_RELEASE_VERSION