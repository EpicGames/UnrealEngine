// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(DEFINE_FORTNITE_SEASON_VERSIONS)
#define FORTNITE_SEASON_VERSION(Version, ID) Version,
#elif defined(CHECK_FORTNITE_SEASON_VERSIONS)
#define FORTNITE_SEASON_VERSION(Version, ID) static_assert(FFortniteSeasonBranchObjectVersion::Version == ID);
#endif

// Before any version changes were made
FORTNITE_SEASON_VERSION(BeforeCustomVersionWasAdded, 0)

// Added FWorldDataLayersActorDesc
FORTNITE_SEASON_VERSION(AddedWorldDataLayersActorDesc, 1)

// Fixed FDataLayerInstanceDesc
FORTNITE_SEASON_VERSION(FixedDataLayerInstanceDesc, 2)

// Serialize DataLayerAssets in WorldPartitionActorDesc
FORTNITE_SEASON_VERSION(WorldPartitionActorDescSerializeDataLayerAssets, 3)

// Remapped bEvaluateWorldPositionOffset to bEvaluateWorldPositionOffsetInRayTracing
FORTNITE_SEASON_VERSION(RemappedEvaluateWorldPositionOffsetInRayTracing, 4)

// Serialize native and base class for actor descriptors
FORTNITE_SEASON_VERSION(WorldPartitionActorDescNativeBaseClassSerialization, 5)

// Serialize tags for actor descriptors
FORTNITE_SEASON_VERSION(WorldPartitionActorDescTagsSerialization, 6)

// Serialize property map for actor descriptors
FORTNITE_SEASON_VERSION(WorldPartitionActorDescPropertyMapSerialization, 7)

// Added ability to mark shapes as probes
FORTNITE_SEASON_VERSION(AddShapeIsProbe, 8)

// Transfer PhysicsAsset SolverSettings (iteration counts etc) to new structure
FORTNITE_SEASON_VERSION(PhysicsAssetNewSolverSettings, 9)
		
// Chaos GeometryCollection now saves levels attribute values
FORTNITE_SEASON_VERSION(ChaosGeometryCollectionSaveLevelsAttribute, 10)

// Serialize actor transform for actor descriptors
FORTNITE_SEASON_VERSION(WorldPartitionActorDescActorTransformSerialization, 11)

// Changing Chaos::FImplicitObjectUnion to store an int32 vs a uint16 for NumLeafObjects.
FORTNITE_SEASON_VERSION(ChaosImplicitObjectUnionLeafObjectsToInt32, 12)

// Chaos Visual Debugger : Adding serialization for properties that were being recorded, but not serialized
FORTNITE_SEASON_VERSION(CVDSerializationFixMissingSerializationProperties, 13)

// Updated Enhanceed Input Mapping Contexts to support adding "Profile override" mappings.
FORTNITE_SEASON_VERSION(EnhancedInputMappingContextProfileMappingsUpdate, 14)

// Introduce per entity support for external owned entities
FORTNITE_SEASON_VERSION(SceneGraphExternalOwnedEntities, 15)

// -----<new versions can be added above this line>-------------------------------------------------
#undef FORTNITE_SEASON_VERSION