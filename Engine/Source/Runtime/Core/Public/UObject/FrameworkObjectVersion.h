// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes made in Dev-Framework stream
struct FFrameworkObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// BodySetup's default instance collision profile is used by default when creating a new instance.
		UseBodySetupCollisionProfile,

		// Regenerate subgraph arrays correctly in animation blueprints to remove duplicates and add
		// missing graphs that appear read only when edited
		AnimBlueprintSubgraphFix,

		// Static and skeletal mesh sockets now use the specified scale
		MeshSocketScaleUtilization,

		// Attachment rules are now explicit in how they affect location, rotation and scale
		ExplicitAttachmentRules,

		// Moved compressed anim data from uasset to the DDC
		MoveCompressedAnimDataToTheDDC,

		// Some graph pins created using legacy code seem to have lost the RF_Transactional flag,
		// which causes issues with undo. Restore the flag at this version
		FixNonTransactionalPins,

		// Create new struct for SmartName, and use that for CurveName
		SmartNameRefactor,
		
		// Add Reference Skeleton to Rig
		AddSourceReferenceSkeletonToRig, 

		// Refactor ConstraintInstance so that we have an easy way to swap behavior paramters
		ConstraintInstanceBehaviorParameters,

		// Pose Asset support mask per bone
		PoseAssetSupportPerBoneMask,

		// Physics Assets now use SkeletalBodySetup instead of BodySetup
		PhysAssetUseSkeletalBodySetup,

		// Remove SoundWave CompressionName
		RemoveSoundWaveCompressionName,

		// Switched render data for clothing over to unreal data, reskinned to the simulation mesh
		AddInternalClothingGraphicalSkinning,

		// Wheel force offset is now applied at the wheel instead of vehicle COM
		WheelOffsetIsFromWheel,

		// Move curve metadata to be saved in skeleton
		// Individual asset still saves some flag - i.e. disabled curve and editable or not, but 
		// major flag - i.e. material types - moves to skeleton and handle in one place
		MoveCurveTypesToSkeleton,
		
		// Cache destructible overlaps on save
		CacheDestructibleOverlaps,

		// Added serialization of materials applied to geometry cache objects
		GeometryCacheMissingMaterials,

		// Switch static & skeletal meshes to calculate LODs based on resolution-independent screen size
		LODsUseResolutionIndependentScreenSize,

		// Blend space post load verification
		BlendSpacePostLoadSnapToGrid,

		// Addition of rate scales to blend space samples
		SupportBlendSpaceRateScale,

		// LOD hysteresis also needs conversion from the LODsUseResolutionIndependentScreenSize version
		LODHysteresisUseResolutionIndependentScreenSize,

		// AudioComponent override subtitle priority default change
		ChangeAudioComponentOverrideSubtitlePriorityDefault,

		// Serialize hard references to sound files when possible
		HardSoundReferences,

		// Enforce const correctness in Animation Blueprint function graphs
		EnforceConstInAnimBlueprintFunctionGraphs,
		
		// Upgrade the InputKeySelector to use a text style
		InputKeySelectorTextStyle,

		// Represent a pins container type as an enum not 3 independent booleans
		EdGraphPinContainerType,

		// Switch asset pins to store as string instead of hard object reference
		ChangeAssetPinsToString,

		// Fix Local Variables so that the properties are correctly flagged as blueprint visible
		LocalVariablesBlueprintVisible,

		// Stopped serializing UField_Next so that UFunctions could be serialized in dependently of a UClass
		// in order to allow us to do all UFunction loading in a single pass (after classes and CDOs are created):
		RemoveUField_Next,
		
		// Fix User Defined structs so that all members are correct flagged blueprint visible 
		UserDefinedStructsBlueprintVisible,

		// FMaterialInput and FEdGraphPin store their name as FName instead of FString
		PinsStoreFName,

		// User defined structs store their default instance, which is used for initializing instances
		UserDefinedStructsStoreDefaultInstance,

		// Function terminator nodes serialize an FMemberReference rather than a name/class pair
		FunctionTerminatorNodesUseMemberReference,

		// Custom event and non-native interface event implementations add 'const' to reference parameters
		EditableEventsUseConstRefParameters,

		// No longer serialize the legacy flag that indicates this state, as it is now implied since we don't serialize the skeleton CDO
		BlueprintGeneratedClassIsAlwaysAuthoritative,

		// Enforce visibility of blueprint functions - e.g. raise an error if calling a private function from another blueprint:
		EnforceBlueprintFunctionVisibility,

		// ActorComponents now store their serialization index
		StoringUCSSerializationIndex,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

private:
	FFrameworkObjectVersion() {}
};
