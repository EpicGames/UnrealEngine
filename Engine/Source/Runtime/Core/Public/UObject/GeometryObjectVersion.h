// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes made in Private-Geometry stream
struct FGeometryObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,
		// Compress Geometry Cache Flipbooks to save disk space
		CompressGeometryCache,
		// Fix for serializing in Mesh vertices for new DynamicMeshVertex layout
		DynamicMeshVertexLayoutChange,
		//Added support for explicit motion vectors
		ExplicitMotionVectors,
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

private:
	FGeometryObjectVersion() {}
};
