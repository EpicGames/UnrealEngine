// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

struct FGuid;

// Custom serialization version for changes made in LoadTimes stream
struct FLoadTimesObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// Allows uncompressed reflection captures for cooked builds
		UncompressedReflectionCapturesForCookedBuilds,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

private:
	FLoadTimesObjectVersion() {}
};
