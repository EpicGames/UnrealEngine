// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes made in Dev-Mobile stream
struct FMobileObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// Removed LightmapUVBias, ShadowmapUVBias from per-instance data
		InstancedStaticMeshLightmapSerialization,

		// Added stationary point/spot light direct contribution to volumetric lightmaps. 
		LQVolumetricLightmapLayers,
		
		// Store Reflection Capture in compressed format for mobile
		StoreReflectionCaptureCompressedMobile,
		
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

private:
	FMobileObjectVersion() {}
};
