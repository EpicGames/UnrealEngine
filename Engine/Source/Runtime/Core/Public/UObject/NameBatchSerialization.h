// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "UObject/NameTypes.h"
#include "Templates/Function.h"

#define ALLOW_NAME_BATCH_SAVING PLATFORM_LITTLE_ENDIAN

//////////////////////////////////////////////////////////////////////////

#if ALLOW_NAME_BATCH_SAVING
// Save display entries in given order to a name blob and a versioned hash blob.
CORE_API void SaveNameBatch(TArrayView<const FDisplayNameEntryId> Names, TArray<uint8>& OutNameData, TArray<uint8>& OutHashData);

// Save display entries in given order to an archive
CORE_API void SaveNameBatch(TArrayView<const FDisplayNameEntryId> Names, FArchive& Out);
#endif

//////////////////////////////////////////////////////////////////////////

// Reserve memory in preparation for batch loading
//
// @param Bytes for existing and new names.
CORE_API void ReserveNameBatch(uint32 NameDataBytes, uint32 HashDataBytes);

enum class ENameBatchLoadingFlags : uint32 {
	None         = 0,
	RespectOrder = 1 << 0,
};
ENUM_CLASS_FLAGS(ENameBatchLoadingFlags);

// Load a name blob with precalculated hashes.
//
// Names are rehashed if hash algorithm version doesn't match.
//
// @param NameData, HashData must be 8-byte aligned.
CORE_API void LoadNameBatch(TArray<FDisplayNameEntryId>& OutNames, TArrayView<const uint8> NameData, TArrayView<const uint8> HashData, ENameBatchLoadingFlags Flags = ENameBatchLoadingFlags::None);

// Load names and precalculated hashes from an archive
//
// Names are rehashed if hash algorithm version doesn't match.
CORE_API TArray<FDisplayNameEntryId> LoadNameBatch(FArchive& Ar, ENameBatchLoadingFlags Flags = ENameBatchLoadingFlags::None);

// Load names and precalculated hashes from an archive using multiple workers
//
// May load synchronously in some cases, like small batches.
//
// Names are rehashed if hash algorithm version doesn't match.
//
// @param Ar is drained synchronously
// @param MaxWorkers > 0
// @return function that waits before returning result, like a simple future.
CORE_API TFunction<TArray<FDisplayNameEntryId>()> LoadNameBatchAsync(FArchive& Ar, uint32 MaxWorkers, ENameBatchLoadingFlags Flags = ENameBatchLoadingFlags::None);

//////////////////////////////////////////////////////////////////////////