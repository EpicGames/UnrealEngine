// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"
#include "Containers/StringConv.h"
#include "Containers/StringFwd.h"
#include "UObject/UnrealNames.h"
#include "Templates/Atomic.h"
#include "Serialization/MemoryLayout.h"
#include "Misc/IntrusiveUnsetOptionalState.h"
#include "Misc/StringBuilder.h"
#include "Misc/UEOps.h"
#include "Trace/Trace.h"

/*----------------------------------------------------------------------------
	Definitions.
----------------------------------------------------------------------------*/

/** 
 * Do we want to support case-variants for FName?
 * This will add an extra NAME_INDEX variable to FName, but means that ToString() will return you the exact same 
 * string that FName::Init was called with (which is useful if your FNames are shown to the end user)
 * Currently this is enabled for the Editor and any Programs (such as UHT), but not the Runtime
 */
#ifndef WITH_CASE_PRESERVING_NAME
	#define WITH_CASE_PRESERVING_NAME WITH_EDITORONLY_DATA
#endif

// Should the number part of the fname be stored in the name table or in the FName instance?
// Storing it in the name table may save memory overall but new number suffixes will cause the name table to grow like unique strings do.
#ifndef UE_FNAME_OUTLINE_NUMBER
	#define UE_FNAME_OUTLINE_NUMBER 0
#endif // UE_FNAME_OUTLINE_NUMBER

// Allows overriding the alignment of FNameEntry. This decreases the granularity of memory allocation for FNames and 
// therefore allows more data to be stored as FNames with the potential of increased waste.
// See the details of FNameEntryId encoding.
// Setting this to a nonzero value less than the natural alignment of FNameEntry will cause a compiler error.
#ifndef UE_FNAME_ENTRY_ALIGNMENT
	#if WITH_EDITOR
		#define UE_FNAME_ENTRY_ALIGNMENT 8
	#else
		#define UE_FNAME_ENTRY_ALIGNMENT 0
	#endif
#endif // UE_FNAME_ENTRY_ALIGNMENT

class FText;

/** Maximum size of name, including the null terminator. */
enum {NAME_SIZE	= 1024};

struct FMinimalName;
struct FScriptName;
struct FNumberedEntry;
class FName;

// Generic constructor/initializer struct to make Clang keep type debug info needed for debug visualizers
struct FClangKeepDebugInfo {};

/** Opaque id to a deduplicated name */
struct FNameEntryId
{
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = FNameEntryId;
	
	// Default initialize to be equal to NAME_None
	constexpr FNameEntryId() : Value(0) {}
	constexpr FNameEntryId(ENoInit) {}
	constexpr explicit FNameEntryId(FIntrusiveUnsetOptionalState) : Value(~0u) {}

	constexpr bool IsNone() const
	{
		return Value == 0;
	}

	/** Slow alphabetical order that is stable / deterministic over process runs, ignores case */
	CORE_API int32 CompareLexical(FNameEntryId Rhs) const;
	bool LexicalLess(FNameEntryId Rhs) const { return CompareLexical(Rhs) < 0; }

	/** Slow alphabetical order that is stable / deterministic over process runs, case-sensitive */
	CORE_API int32 CompareLexicalSensitive(FNameEntryId Rhs) const;
	bool LexicalSensitiveLess(FNameEntryId Rhs) const { return CompareLexicalSensitive(Rhs) < 0; }

	/** Fast non-alphabetical order that is only stable during this process' lifetime */
	int32 CompareFast(FNameEntryId Rhs) const { return Value - Rhs.Value; };
	bool FastLess(FNameEntryId Rhs) const { return CompareFast(Rhs) < 0; }

	/** Fast non-alphabetical order that is only stable during this process' lifetime */
	bool UEOpLessThan(FNameEntryId Rhs) const { return Value < Rhs.Value; }

	/** Fast non-alphabetical order that is only stable during this process' lifetime */
	bool UEOpEquals(FNameEntryId Rhs) const { return Value == Rhs.Value; }
	
	/** Comparison against special type for checking TOptional<FName>:IsSet */
	bool UEOpEquals(FIntrusiveUnsetOptionalState) const
	{
		return Value == ~0u;
	}

	// Returns true if this FNameEntryId is not equivalent to NAME_None
	constexpr explicit operator bool() const { return Value != 0; }

	/** Get process specific integer */
	constexpr uint32 ToUnstableInt() const { return Value; }

	/** Create from unstable int produced by this process */
	static FNameEntryId FromUnstableInt(uint32 UnstableInt)
	{
		FNameEntryId Id;
		Id.Value = UnstableInt;
		return Id;
	}

	FORCEINLINE static FNameEntryId FromEName(EName Ename)
	{
		return Ename == NAME_None ? FNameEntryId() : FromValidEName(Ename);
	}

	inline bool UEOpEquals(EName Ename) const
	{
		return Ename == NAME_None ? Value == 0 : Value == FromValidENamePostInit(Ename).Value;
	}

private:
	uint32 Value;

	CORE_API static FNameEntryId FromValidEName(EName Ename);
	CORE_API static FNameEntryId FromValidENamePostInit(EName Ename);

public:
	friend uint32 GetTypeHash(FNameEntryId Id)
	{
		const uint32 UnstableInt = Id.ToUnstableInt();
		return HashCombineFast(UnstableInt, UnstableInt);
	}

#if !UE_FNAME_OUTLINE_NUMBER
	friend uint32 GetTypeHashWithNumber(FNameEntryId Id, uint32 Number)
	{
		const uint32 UnstableInt = Id.ToUnstableInt();
		return HashCombineFast(UnstableInt + Number, UnstableInt);
	}
#endif

	/** Serialize as process specific unstable int */
	CORE_API friend FArchive& operator<<(FArchive& Ar, FNameEntryId& InId);
};

/**
 * Legacy typedef - this is no longer an index
 *
 * Use GetTypeHash(FName) or GetTypeHash(FNameEntryId) for hashing
 * To compare with ENames use FName(EName) or FName::ToEName() instead
 */
typedef FNameEntryId NAME_INDEX;

#define checkName checkSlow

/** Externally, the instance number to represent no instance number is NAME_NO_NUMBER, 
    but internally, we add 1 to indices, so we use this #define internally for 
	zero'd memory initialization will still make NAME_None as expected */
#define NAME_NO_NUMBER_INTERNAL	0

/** Conversion routines between external representations and internal */
#define NAME_INTERNAL_TO_EXTERNAL(x) (x - 1)
#define NAME_EXTERNAL_TO_INTERNAL(x) (x + 1)

/** Special value for an FName with no number */
#define NAME_NO_NUMBER NAME_INTERNAL_TO_EXTERNAL(NAME_NO_NUMBER_INTERNAL)


/** this is the character used to separate a subobject root from its subobjects in a path name. */
#define SUBOBJECT_DELIMITER_ANSI		":"
#define SUBOBJECT_DELIMITER				TEXT(SUBOBJECT_DELIMITER_ANSI)

/** this is the character used to separate a subobject root from its subobjects in a path name, as a char */
#define SUBOBJECT_DELIMITER_CHAR_ANSI	':'
#define SUBOBJECT_DELIMITER_CHAR		TEXT(SUBOBJECT_DELIMITER_CHAR_ANSI)

/** These are the characters that cannot be used in general FNames */
#define INVALID_NAME_CHARACTERS			TEXT("\"' ,\n\r\t")

/** These characters cannot be used in object names */
#define INVALID_OBJECTNAME_CHARACTERS	TEXT("\"' ,/.:|&!~\n\r\t@#(){}[]=;^%$`")

/** These characters cannot be used in ObjectPaths, which includes both the package path and part after the first . */
#define INVALID_OBJECTPATH_CHARACTERS	TEXT("\"' ,|&!~\n\r\t@#(){}[]=;^%$`")

/** These characters cannot be used in long package names */
#define INVALID_LONGPACKAGE_CHARACTERS	TEXT("\\:*?\"<>|' ,.&!~\n\r\t@#")

/** These characters can be used in relative directory names (lowercase versions as well) */
#define VALID_SAVEDDIRSUFFIX_CHARACTERS	TEXT("_0123456789ABCDEFGHIJKLMNOPQRSTUVWXYZabcdefghijklmnopqrstuvwxyz")

/** Delimiter used to distinguish when a named pak chunk starts and ends. When using named pak chunk this will be the character wrapping the name. */
#define NAMED_PAK_CHUNK_DELIMITER_CHAR	TEXT('-')

enum class ENameCase : uint8
{
	CaseSensitive,
	IgnoreCase,
};

enum ELinkerNameTableConstructor    {ENAME_LinkerConstructor};

/** Enumeration for finding name. */
enum EFindName
{
	/** 
	* Find a name; return 0/NAME_None/FName() if it doesn't exist.
	* When UE_FNAME_OUTLINE_NUMBER is set, we search for the exact name including the number suffix.
	* Otherwise we search only for the string part.
	*/
	FNAME_Find,

	/** Find a name or add it if it doesn't exist. */
	FNAME_Add
};

namespace UE::Core::Private
{

/**
 * Parse the number from the end of a string like Prefix_1.
 *
 * Number must be non-negative, less than MAX_int32, and not have a leading 0 unless the number is 0.
 * Returns the internal representation of the external number in the string. Example: Prefix_0 returns 1.
 *
 * @param Name   Name to parse the trailing number from.
 * @param InOutLen   Length of the input name, updated to exclude the _<N> suffix if a number is parsed.
 * @return Internal value of the number that was parsed, or NAME_NO_NUMBER_INTERNAL if no number was parsed.
 */
template <typename CharType>
static constexpr uint32 ParseNumberFromName(const CharType* Name, int32& InOutLen)
{
	int32 Digits = 0;
	for (const CharType* It = Name + InOutLen - 1; It >= Name && *It >= '0' && *It <= '9'; --It)
	{
		++Digits;
	}

	const CharType* FirstDigit = Name + InOutLen - Digits;
	constexpr int32 MaxDigitsInt32 = 10;
	if (Digits && Digits < InOutLen && FirstDigit[-1] == '_' && Digits <= MaxDigitsInt32 && (Digits == 1 || *FirstDigit != '0'))
	{
		int64 Number = 0;
		for (int32 Index = 0; Index < Digits; ++Index)
		{
			Number = 10 * Number + (FirstDigit[Index] - '0');
		}
		if (Number < MAX_int32)
		{
			InOutLen -= 1 + Digits;
			return static_cast<uint32>(NAME_EXTERNAL_TO_INTERNAL(Number));
		}
	}

	return NAME_NO_NUMBER_INTERNAL;
}

} // UE::Core::Private

/*----------------------------------------------------------------------------
	FNameEntry.
----------------------------------------------------------------------------*/

/** Implementation detail exposed for debug visualizers */
struct FNameEntryHeader
{
	uint16 bIsWide : 1;
#if WITH_CASE_PRESERVING_NAME
	uint16 Len : 15;
#else
	static constexpr inline uint32 ProbeHashBits = 5;
	uint16 LowercaseProbeHash : ProbeHashBits;
	uint16 Len : 10;
#endif
};

/**
 * A global deduplicated name stored in the global name table.
 */
struct alignas(UE_FNAME_ENTRY_ALIGNMENT) FNameEntry
{
private:
#if WITH_CASE_PRESERVING_NAME
	FNameEntryId ComparisonId;
#endif
	FNameEntryHeader Header;

	// Unaligned to reduce alignment waste for non-numbered entries
	struct FNumberedData
	{
#if UE_FNAME_OUTLINE_NUMBER	
#if WITH_CASE_PRESERVING_NAME // ComparisonId is 4B-aligned, 4B-align Id/Number by 2B pad after 2B Header
		uint8 Pad[sizeof(Header) % alignof(decltype(ComparisonId))]; 
#endif						
		uint8 Id[sizeof(FNameEntryId)];
		uint8 Number[sizeof(uint32)];
#endif // UE_FNAME_OUTLINE_NUMBER	
	};

	union
	{
		ANSICHAR			AnsiName[NAME_SIZE];
		WIDECHAR			WideName[NAME_SIZE];
		uint8				NameData[0];
		FNumberedData		NumberedName;
	};


	FNameEntry(FClangKeepDebugInfo);
	FNameEntry(const FNameEntry&) = delete;
	FNameEntry(FNameEntry&&) = delete;
	FNameEntry& operator=(const FNameEntry&) = delete;
	FNameEntry& operator=(FNameEntry&&) = delete;

public:
	/** Returns whether this name entry is represented via WIDECHAR or ANSICHAR. */
	FORCEINLINE bool IsWide() const								{ return Header.bIsWide; }
	FORCEINLINE int32 GetNameLength() const 					{ return Header.Len; }
	CORE_API int32 GetNameLengthUtf8() const;
#if UE_FNAME_OUTLINE_NUMBER
	FORCEINLINE bool IsNumbered() const 						{ return Header.Len == 0; }
#else
	FORCEINLINE bool IsNumbered() const 						{ return false; }
#endif

	/** Copy unterminated name to TCHAR buffer without allocating. */
	CORE_API void GetUnterminatedName(TCHAR* OutName, uint32 OutSize) const;

	/** Copy null-terminated name to TCHAR buffer without allocating. */
	CORE_API void GetName(TCHAR(&OutName)[NAME_SIZE]) const;

	/** Copy null-terminated name to ANSICHAR buffer without allocating. Entry must not be wide. */
	CORE_API void GetAnsiName(ANSICHAR(&OutName)[NAME_SIZE]) const;

	/** Copy null-terminated name to WIDECHAR buffer without allocating. Entry must be wide. */
	CORE_API void GetWideName(WIDECHAR(&OutName)[NAME_SIZE]) const;

	/** Copy name to a dynamically allocated FString. */
	CORE_API FString GetPlainNameString() const;

	/** Get name without number part as a dynamically allocated shared string */
	[[nodiscard]] CORE_API UE::FSharedString GetPlainNameSharedString() const;

	/** Copy name to a dynamically allocated FString. */
	CORE_API FUtf8String GetPlainNameUtf8String() const;

	/** Appends name to string. May allocate. */
	CORE_API void AppendNameToString(FWideString& OutString) const;
	CORE_API void AppendNameToString(FUtf8String& OutString) const;

	/** Appends name to string builder. */
	CORE_API void AppendNameToString(FWideStringBuilderBase& OutString) const;
	CORE_API void AppendNameToString(FUtf8StringBuilderBase& OutString) const;

	/** Appends name to string builder. Entry must not be wide. */
	CORE_API void AppendAnsiNameToString(FAnsiStringBuilderBase& OutString) const;

	/** Appends name to string with path separator using FString::PathAppend(). */
	CORE_API void AppendNameToPathString(FString& OutString) const;

	CORE_API void DebugDump(FOutputDevice& Out) const;

	CORE_API int32 GetSizeInBytes() const;

	CORE_API void Write(FArchive& Ar) const;

	static constexpr int32 GetDataOffset() { return offsetof(FNameEntry, NameData); }
	struct CORE_API FNameStringView MakeView(union FNameBuffer& OptionalDecodeBuffer) const;
private:
	friend class FName;
	friend struct FNameHelper;
	friend class FNameEntryAllocator;
	friend class FNamePoolShardBase;
	friend class FNamePool;
	void CopyUnterminatedName(ANSICHAR* OutName, uint32 Len) const;
	void CopyUnterminatedName(WIDECHAR* OutName, uint32 Len) const;
	void CopyAndConvertUnterminatedName(TCHAR* OutName, uint32 Len) const;
	const ANSICHAR* GetUnterminatedName(ANSICHAR(&OptionalDecodeBuffer)[NAME_SIZE]) const;
	const WIDECHAR* GetUnterminatedName(WIDECHAR(&OptionalDecodeBuffer)[NAME_SIZE]) const;

#if UE_FNAME_OUTLINE_NUMBER
	// @pre IsNumbered()
	FORCEINLINE const FNumberedEntry& GetNumberedName() const	{ return reinterpret_cast<const FNumberedEntry&>(NumberedName.Id[0]); }
	uint32 GetNumber() const;
#endif // UE_FNAME_OUTLINE_NUMBER
};

/**
 *  This struct is only used during loading/saving and is not part of the runtime costs
 */
struct FNameEntrySerialized
{
	bool bIsWide = false;

	union
	{
		ANSICHAR	AnsiName[NAME_SIZE];
		WIDECHAR	WideName[NAME_SIZE];
	};

	// These are not used anymore but recalculated on save to maintain serialization format
	uint16 NonCasePreservingHash = 0;
	uint16 CasePreservingHash = 0;

	FNameEntrySerialized(const FNameEntry& NameEntry);
	FNameEntrySerialized(enum ELinkerNameTableConstructor) {}

	/**
	 * Returns direct access to null-terminated name if narrow
	 */
	ANSICHAR const* GetAnsiName() const
	{
		check(!bIsWide);
		return AnsiName;
	}

	/**
	 * Returns direct access to null-terminated name if wide
	 */
	WIDECHAR const* GetWideName() const
	{
		check(bIsWide);
		return WideName;
	}

	/**
	 * Returns FString of name portion minus number.
	 */
	CORE_API FString GetPlainNameString() const;	

	friend CORE_API FArchive& operator<<(FArchive& Ar, FNameEntrySerialized& E);
	friend FArchive& operator<<(FArchive& Ar, FNameEntrySerialized* E)
	{
		return Ar << *E;
	}
};

/**
 * The minimum amount of data required to reconstruct a name
 * This is smaller than FName when WITH_CASE_PRESERVING_NAME is set, but you lose the case-preserving behavior.
 * The size of this type is not portable across different platforms and configurations, as with FName itself.
 */
struct FMinimalName
{
	friend FName;

	FMinimalName() {}
	
	FMinimalName(EName N)
		: Index(FNameEntryId::FromEName(N))
	{
	}

	FORCEINLINE explicit FMinimalName(const FName& Name);
	FORCEINLINE bool IsNone() const;
	FORCEINLINE bool UEOpLessThan(FMinimalName Rhs) const;
	
private:
	/** Index into the Names array (used to find String portion of the string/number pair) */
	FNameEntryId	Index;
#if !UE_FNAME_OUTLINE_NUMBER
	/** Number portion of the string/number pair (stored internally as 1 more than actual, so zero'd memory will be the default, no-instance case) */
	int32			Number = NAME_NO_NUMBER_INTERNAL;
#endif // UE_FNAME_OUTLINE_NUMBE

public:
#if UE_FNAME_OUTLINE_NUMBER
	FORCEINLINE bool UEOpEquals(FMinimalName Rhs) const
	{
		return Index == Rhs.Index;
	}
	friend FORCEINLINE uint32 GetTypeHash(FMinimalName Name)
	{
		return GetTypeHash(Name.Index);
	}
#else
	FORCEINLINE bool UEOpEquals(FMinimalName Rhs) const
	{
		return Index == Rhs.Index && Number == Rhs.Number;
	}
	friend FORCEINLINE uint32 GetTypeHash(FMinimalName Name)
	{
		return GetTypeHashWithNumber(Name.Index, Name.Number);
	}
#endif
};

/**
 * The full amount of data required to reconstruct a case-preserving name
 * This will be the maximum size of an FName across all values of WITH_CASE_PRESERVING_NAME and UE_FNAME_OUTLINE_NUMBER
 * and is used to store an FName in cases where  the size of a name must be constant between build configurations (eg, blueprint bytecode)
 * the layout is not guaranteed to be the same as FName even if the size is the same, so memory cannot be reinterpreted between the two.
 * The layout here must be as expected by FScriptBytecodeWriter and XFER_NAME
 */
struct FScriptName
{
	friend FName;

	FScriptName() {}
	
	FScriptName(EName Ename)
		: ComparisonIndex(FNameEntryId::FromEName(Ename))
		, DisplayIndex(ComparisonIndex)
	{
	}

	FORCEINLINE explicit FScriptName(const FName& Name);
	FORCEINLINE bool IsNone() const;
	inline bool UEOpEquals(EName Name) const { return UEOpEquals(FScriptName(Name)); }

	CORE_API FString ToString() const;
	CORE_API FUtf8String ToUtf8String() const;

	// The internal structure of FScriptName is private in order to handle UE_FNAME_OUTLINE_NUMBER
private: 
	/** Encoded address of name entry  (used to find String portion of the string/number pair used for comparison) */
	FNameEntryId	ComparisonIndex;
	/** Encoded address of name entry  (used to find String portion of the string/number pair used for display) */
	FNameEntryId	DisplayIndex;
#if UE_FNAME_OUTLINE_NUMBER
	uint32			Dummy = 0; // Dummy to keep the size the same regardless of build configuration, but change the name so trying to use Number is a compile error
#else // UE_FNAME_OUTLINE_NUMBER
	/** Number portion of the string/number pair (stored internally as 1 more than actual, so zero'd memory will be the default, no-instance case) */
	uint32			Number = NAME_NO_NUMBER_INTERNAL;
#endif // UE_FNAME_OUTLINE_NUMBER

public:
#if UE_FNAME_OUTLINE_NUMBER
	FORCEINLINE bool UEOpEquals(FScriptName Rhs) const
	{
		return ComparisonIndex == Rhs.ComparisonIndex;
	}
	friend FORCEINLINE uint32 GetTypeHash(FScriptName Name)
	{
		return GetTypeHash(Name.ComparisonIndex);
	}
#else
	FORCEINLINE bool UEOpEquals(FScriptName Rhs) const
	{
		return ComparisonIndex == Rhs.ComparisonIndex && Number == Rhs.Number;
	}
	friend FORCEINLINE uint32 GetTypeHash(FScriptName Name)
	{
		return GetTypeHashWithNumber(Name.ComparisonIndex, Name.Number);
	}
#endif
};

struct FMemoryImageName;

namespace Freeze
{
	CORE_API void ApplyMemoryImageNamePatch(void* NameDst, const FMemoryImageName& Name, const FPlatformTypeLayoutParameters& LayoutParams);
	CORE_API void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const FMemoryImageName& Object, const FTypeLayoutDesc&);
	CORE_API uint32 IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const FMemoryImageName& Object, void* OutDst);
}

/**
 * Predictably sized structure for representing an FName in memory images while allowing the size to be smaller than FScriptName
 * when case-preserving behavior is not required.
 */
struct FMemoryImageName
{
	friend FName;

	friend CORE_API void Freeze::ApplyMemoryImageNamePatch(void* NameDst, const FMemoryImageName& Name, const FPlatformTypeLayoutParameters& LayoutParams);
	friend CORE_API void Freeze::IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const FMemoryImageName& Object, const FTypeLayoutDesc&);
	friend CORE_API uint32 Freeze::IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const FMemoryImageName& Object, void* OutDst);

	FMemoryImageName();
	FMemoryImageName(EName Name);
	FORCEINLINE FMemoryImageName(const FName& Name);

	inline bool UEOpEquals(EName Name) const { return UEOpEquals(FMemoryImageName(Name)); }

	FORCEINLINE bool IsNone() const;
	CORE_API FString ToString() const;

	// The internal structure of FMemoryImageName is private in order to handle UE_FNAME_OUTLINE_NUMBER
private:
	/** Encoded address of name entry (used to find String portion of the string/number pair used for comparison) */
	FNameEntryId	ComparisonIndex;
#if UE_FNAME_OUTLINE_NUMBER
	uint32			Dummy = 0; // Dummy to keep the size the same regardless of build configuration, but change the name so trying to use Number is a compile error
#else // UE_FNAME_OUTLINE_NUMBER
	/** Number portion of the string/number pair (stored internally as 1 more than actual, so zero'd memory will be the default, no-instance case) */
	uint32			Number = NAME_NO_NUMBER_INTERNAL;
#endif // UE_FNAME_OUTLINE_NUMBER
#if WITH_CASE_PRESERVING_NAME
	/** Encoded address of name entry (used to find String portion of the string/number pair used for display) */
	FNameEntryId	DisplayIndex;
#endif

public:
#if UE_FNAME_OUTLINE_NUMBER
	FORCEINLINE bool UEOpEquals(FMemoryImageName Rhs) const
	{
		return ComparisonIndex == Rhs.ComparisonIndex;
	}
	friend FORCEINLINE uint32 GetTypeHash(FMemoryImageName Name)
	{
		return GetTypeHash(Name.ComparisonIndex);
	}
#else
	FORCEINLINE bool UEOpEquals(FMemoryImageName Rhs) const
	{
		return ComparisonIndex == Rhs.ComparisonIndex && Number == Rhs.Number;
	}
	friend FORCEINLINE uint32 GetTypeHash(FMemoryImageName Name)
	{
		return GetTypeHashWithNumber(Name.ComparisonIndex, Name.Number);
	}
#endif
};

/**
 * Public name, available to the world.  Names are stored as a combination of
 * an index into a table of unique strings and an instance number.
 * Names are case-insensitive, but case-preserving (when WITH_CASE_PRESERVING_NAME is 1)
 */
class FName
{
public:
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = FName;

#if UE_FNAME_OUTLINE_NUMBER
	[[nodiscard]] CORE_API FNameEntryId GetComparisonIndex() const;
	[[nodiscard]] CORE_API FNameEntryId GetDisplayIndex() const;
#else // UE_FNAME_OUTLINE_NUMBER
	[[nodiscard]] FORCEINLINE FNameEntryId GetComparisonIndex() const
	{
		checkName(IsWithinBounds(ComparisonIndex));
		return ComparisonIndex;
	}

	[[nodiscard]] FORCEINLINE FNameEntryId GetDisplayIndex() const
	{
		const FNameEntryId Index = GetDisplayIndexFast();
		checkName(IsWithinBounds(Index));
		return Index;
	}
#endif //UE_FNAME_OUTLINE_NUMBER

#if UE_FNAME_OUTLINE_NUMBER
	// This should really be named GetInternalNumber().
	[[nodiscard]] CORE_API int32 GetNumber() const;
	// This should really be named SetInternalNumber().
	CORE_API void SetNumber(const int32 NewNumber);
#else //UE_FNAME_OUTLINE_NUMBER
	// This should really be named GetInternalNumber().
	[[nodiscard]] FORCEINLINE int32 GetNumber() const
	{
		return Number;
	}

	// This should really be named SetInternalNumber().
	FORCEINLINE void SetNumber(const int32 NewInternalNumber)
	{
		Number = NewInternalNumber;
	}
#endif //UE_FNAME_OUTLINE_NUMBER
	
	/** Get name without number part as a dynamically allocated string */
	[[nodiscard]] CORE_API FString GetPlainNameString() const;

	/** Convert name without number part into TCHAR buffer and returns string length. Doesn't allocate. */
	CORE_API uint32 GetPlainNameString(TCHAR(&OutName)[NAME_SIZE]) const;

	/** Get name without number part as a dynamically allocated shared string */
	[[nodiscard]] CORE_API UE::FSharedString GetPlainNameSharedString() const;

	/** Copy ANSI name without number part. Must *only* be used for ANSI FNames. Doesn't allocate. */
	CORE_API void GetPlainANSIString(ANSICHAR(&AnsiName)[NAME_SIZE]) const;

	/** Copy wide name without number part. Must *only* be used for wide FNames. Doesn't allocate. */
	CORE_API void GetPlainWIDEString(WIDECHAR(&WideName)[NAME_SIZE]) const;

	[[nodiscard]] CORE_API const FNameEntry* GetComparisonNameEntry() const;
	[[nodiscard]] CORE_API const FNameEntry* GetDisplayNameEntry() const;

	/**
	 * Converts an FName to a readable format
	 *
	 * @return String representation of the name
	 */
	[[nodiscard]] CORE_API FString ToString() const;

	/**
	 * Converts an FName to a readable format
	 *
	 * @return String representation of the name
	 */
	[[nodiscard]] CORE_API FUtf8String ToUtf8String() const;

	/**
	 * Converts an FName to a readable format, in place
	 * 
	 * @param Out String to fill with the string representation of the name
	 */
	CORE_API void ToString(FWideString& Out) const;
	CORE_API void ToString(FUtf8String& Out) const;

	/**
	 * Converts an FName to a readable format, in place
	 * 
	 * @param Out StringBuilder to fill with the string representation of the name
	 */
	CORE_API void ToString(FWideStringBuilderBase& Out) const;
	CORE_API void ToString(FUtf8StringBuilderBase& Out) const;

	/**
	 * Get the number of characters, excluding null-terminator, that ToString() would yield
	 */
	[[nodiscard]] CORE_API uint32 GetStringLength() const;

	/**
	 * Buffer size required for any null-terminated FName string, i.e. [name] '_' [digits] '\0'
	 */
	static constexpr inline uint32 StringBufferSize = NAME_SIZE + 1 + 10; // NAME_SIZE includes null-terminator

private:
	/**
	 * Internal implementation of non-allocating ToString().
	 * OutSize is the number of valid elements in the buffer, including the null terminator. Must be > 0.
	 * Returns the length of the string, excluding the null terminator.
	 */
	CORE_API uint32 ToStringInternal(TCHAR* Out, uint32 OutSize) const;

public:
	/**
	 * Converts the FName to a string buffer, avoiding dynamic allocations.
	 *
	 * Returns the length of the string, excluding the null terminator.
	 *
	 * Note that a default constructed FName returns "None" instead of ""
	 */
	template <uint32 N>
	uint32 ToString(TCHAR (&Out)[N]) const
	{
		UE_STATIC_DEPRECATE(5.6, N < StringBufferSize,
			"FName::ToString(TCHAR (&Out)[N]) requires a buffer of size of at least FName::StringBufferSize. "
			"Use ToStringTruncate() if a smaller buffer is required and it is safe for the returned string to be truncated.");

		return ToStringInternal(Out, N);
	}

	/**
	 * Converts the FName to a string buffer, avoiding dynamic allocations. Truncates the name if it does not fit in the specified output buffer.
	 * Use a buffer size of at least FName::StringBufferSize to avoid truncation.
	 *
	 * Returns the length of the (possibly truncated) string, excluding the null terminator. 
	 *
	 * Note that a default constructed FName returns "None" instead of ""
	 */
	uint32 ToStringTruncate(TCHAR* Out, uint32 OutSize) const
	{
		return ToStringInternal(Out, OutSize);
	}

	template <uint32 N>
	uint32 ToStringTruncate(TCHAR (&Out)[N]) const
	{
		static_assert(N > 0, "Out buffer must have at least one element.");
		return ToStringTruncate(Out, N);
	}

	/**
	 * Converts an FName to a readable format, in place, appending to an existing string (ala GetFullName)
	 * 
	 * @param Out String to append with the string representation of the name
	 */
	CORE_API void AppendString(FWideString& Out) const;
	CORE_API void AppendString(FUtf8String& Out) const;

	/**
	 * Converts an FName to a readable format, in place, appending to an existing string (ala GetFullName)
	 * 
	 * @param Out StringBuilder to append with the string representation of the name
	 */
	CORE_API void AppendString(FWideStringBuilderBase& Out) const;
	CORE_API void AppendString(FUtf8StringBuilderBase& Out) const;

	/**
	 * Converts an ANSI FName to a readable format appended to the string builder.
	 *
	 * @param Out A string builder to write the readable representation of the name into.
	 *
	 * @return Whether the string is ANSI. A return of false indicates that the string was wide and was not written.
	 */
	CORE_API bool TryAppendAnsiString(FAnsiStringBuilderBase& Out) const;

	/**
	 * Check to see if this FName matches the other FName, potentially also checking for any case variations
	 */
	[[nodiscard]] FORCEINLINE bool IsEqual(const FName& Other, const ENameCase CompareMethod = ENameCase::IgnoreCase, const bool bCompareNumber = true) const;

	[[nodiscard]] FORCEINLINE bool UEOpEquals(FName Other) const
	{
		return ToUnstableInt() == Other.ToUnstableInt();
	}

	/** Special comparison operator for TOptional<FName>::IsSet */
	[[nodiscard]] inline bool UEOpEquals(FIntrusiveUnsetOptionalState I) const
	{
		return ComparisonIndex == I;
	}

	/** Fast non-alphabetical order that is only stable during this process' lifetime. */
	[[nodiscard]] FORCEINLINE bool FastLess(const FName& Other) const
	{
		return CompareIndexes(Other) < 0;
	}

	/** Slow alphabetical order that is stable / deterministic over process runs. */
	[[nodiscard]] FORCEINLINE bool LexicalLess(const FName& Other) const
	{
		return Compare(Other) < 0;
	}

	/** True for FName(), FName(NAME_None) and FName("None") */
	[[nodiscard]] FORCEINLINE bool IsNone() const
	{
#if !WITH_CASE_PRESERVING_NAME
		return ToUnstableInt() == 0;
#else
		return ComparisonIndex.IsNone() && GetNumber() == NAME_NO_NUMBER_INTERNAL;
#endif
	}

	/**
	 * Paranoid sanity check
	 *
	 * All FNames are valid except for stomped memory, dangling pointers, etc.
	 * Should only be used to investigate such bugs and not in production code.
	 */
	[[nodiscard]] bool IsValid() const { return IsWithinBounds(ComparisonIndex); }

	/** Paranoid sanity check, same as IsValid() */
	[[nodiscard]] bool IsValidIndexFast() const { return IsValid(); }

	/**
	 * Checks to see if an internal number is valid, such as those provided to the FName(Name, Number) constructors.
	 *
	 * @param	InInternalNumber	The number to check.
	 *
	 * @return	true if the number is valid.
	 */
	[[nodiscard]] static constexpr bool IsValidInternalNumber(int32 InInternalNumber)
	{
		return InInternalNumber >= NAME_NO_NUMBER_INTERNAL;
	}

	/**
	 * Checks to see that a given name-like string follows the rules that Unreal requires.
	 *
	 * @param	InName			String containing the name to test.
	 * @param	InInvalidChars	The set of invalid characters that the name cannot contain.
	 * @param	OutReason		If the check fails, this string is filled in with the reason why.
	 * @param	InErrorCtx		Error context information to show in the error message (default is "Name").
	 *
	 * @return	true if the name is valid
	 */
	CORE_API static bool IsValidXName( const FName InName, const FString& InInvalidChars, FText* OutReason = nullptr, const FText* InErrorCtx = nullptr );
	CORE_API static bool IsValidXName( const TCHAR* InName, const FString& InInvalidChars, FText* OutReason = nullptr, const FText* InErrorCtx = nullptr );
	CORE_API static bool IsValidXName( const FString& InName, const FString& InInvalidChars, FText* OutReason = nullptr, const FText* InErrorCtx = nullptr );
	CORE_API static bool IsValidXName( const FStringView& InName, const FString& InInvalidChars, FText* OutReason = nullptr, const FText* InErrorCtx = nullptr );

	/**
	 * Checks to see that a FName follows the rules that Unreal requires.
	 *
	 * @param	InInvalidChars	The set of invalid characters that the name cannot contain
	 * @param	OutReason		If the check fails, this string is filled in with the reason why.
	 * @param	InErrorCtx		Error context information to show in the error message (default is "Name").
	 *
	 * @return	true if the name is valid
	 */
	bool IsValidXName( const FString& InInvalidChars, FText* OutReason = nullptr, const FText* InErrorCtx = nullptr ) const
	{
		return IsValidXName(*this, InInvalidChars, OutReason, InErrorCtx);
	}
	CORE_API bool IsValidXName() const;

	/**
	 * Takes an FName and checks to see that it follows the rules that Unreal requires.
	 *
	 * @param	OutReason		If the check fails, this string is filled in with the reason why.
	 * @param	InInvalidChars	The set of invalid characters that the name cannot contain
	 *
	 * @return	true if the name is valid
	 */
	bool IsValidXName( FText& OutReason, const FString& InInvalidChars ) const
	{
		return IsValidXName(*this, InInvalidChars, &OutReason);
	}
	CORE_API bool IsValidXName( FText& OutReason ) const;

	/**
	 * Takes an FName and checks to see that it follows the rules that Unreal requires for object names.
	 *
	 * @param	OutReason		If the check fails, this string is filled in with the reason why.
	 *
	 * @return	true if the name is valid
	 */
	CORE_API bool IsValidObjectName( FText& OutReason ) const;

	/**
	 * Takes an FName and checks to see that it follows the rules that Unreal requires for package or group names.
	 *
	 * @param	OutReason		If the check fails, this string is filled in with the reason why.
	 * @param	bIsGroupName	if true, check legality for a group name, else check legality for a package name
	 *
	 * @return	true if the name is valid
	 */
	CORE_API bool IsValidGroupName( FText& OutReason, bool bIsGroupName=false ) const;

	/**
	 * Printing FNames in logging or on screen can be problematic when they contain Whitespace characters such as \n and \r,
	 * so this will return an FName based upon the calling FName, but with any Whitespace characters potentially problematic for
	 * showing in a log or on screen omitted.
	 *
	* @return the new FName based upon the calling FName, but with any Whitespace characters potentially problematic for
	 * showing in a log or on screen omitted.
	 */
	[[nodiscard]] CORE_API static FString SanitizeWhitespace(const FString& FNameString);
	
	/**
	 * Compares name to passed in one. Sort is alphabetical ascending.
	 *
	 * @param	Other	Name to compare this against
	 * @return	< 0 is this < Other, 0 if this == Other, > 0 if this > Other
	 */
	[[nodiscard]] CORE_API int32 Compare( const FName& Other ) const;

	/**
	 * Compares name to passed in one. Sort is case-sensitive and alphabetical ascending.
	 *
	 * @param	Other	Name to compare this against
	 * @return	< 0 is this < Other, 0 if this == Other, > 0 if this > Other
	 */
	[[nodiscard]] CORE_API int32 CompareSensitive( const FName& Other ) const;

	/**
	 * Fast non-alphabetical order that is only stable during this process' lifetime.
	 *
	 * @param	Other	Name to compare this against
	 * @return	< 0 is this < Other, 0 if this == Other, > 0 if this > Other
	 */
	[[nodiscard]] FORCEINLINE int32 CompareIndexes(const FName& Other) const
	{
		if (int32 ComparisonDiff = ComparisonIndex.CompareFast(Other.ComparisonIndex))
		{
			return ComparisonDiff;
		}

#if UE_FNAME_OUTLINE_NUMBER
		return 0;  // If comparison indices are the same we are the same
#else //UE_FNAME_OUTLINE_NUMBER
		return GetNumber() - Other.GetNumber();
#endif //UE_FNAME_OUTLINE_NUMBER
	}

	/**
	 * Create an FName with a hardcoded string index.
	 *
	 * @param N The hardcoded value the string portion of the name will have. The number portion will be NAME_NO_NUMBER
	 */
	FORCEINLINE FName(EName Ename) : FName(Ename, NAME_NO_NUMBER_INTERNAL) {}

	/**
	 * Create an FName with a hardcoded string index and (instance).
	 *
	 * @param N The hardcoded value the string portion of the name will have
	 * @param InInternalNumber The hardcoded value for the number portion of the name, in internal format
	 */
	FORCEINLINE FName(EName Ename, int32 InInternalNumber)
		: FName(CreateNumberedNameIfNecessary(FNameEntryId::FromEName(Ename), InInternalNumber))
	{
	}


	/**
	 * Create an FName from an existing string, but with a different instance.
	 *
	 * @param Other The FName to take the string values from
	 * @param InInternalNumber The hardcoded value for the number portion of the name, in internal format
	 */
	FORCEINLINE FName(FName Other, int32 InInternalNumber)
		: FName(CreateNumberedNameIfNecessary(Other.GetComparisonIndex(), Other.GetDisplayIndex(), InInternalNumber))
	{
	}

	/**
	 * Create an FName from its component parts
	 * Only call this if you *really* know what you're doing
	 */
	FORCEINLINE FName(FNameEntryId InComparisonIndex, FNameEntryId InDisplayIndex, int32 InInternalNumber)
		: FName(CreateNumberedNameIfNecessary(InComparisonIndex, InDisplayIndex, InInternalNumber))
	{
	}

#if WITH_CASE_PRESERVING_NAME
	[[nodiscard]] CORE_API static FNameEntryId GetComparisonIdFromDisplayId(FNameEntryId DisplayId);
#else
	[[nodiscard]] static FNameEntryId GetComparisonIdFromDisplayId(FNameEntryId DisplayId) { return DisplayId; }
#endif

	/**
	 * Only call this if you *really* know what you're doing
	 */
	[[nodiscard]] static FName CreateFromDisplayId(FNameEntryId DisplayId, int32 InInternalNumber)
	{
#if UE_FNAME_OUTLINE_NUMBER
		checkSlow(ResolveEntry(DisplayId)->IsNumbered() == false); // This id should be unnumbered i.e. returned from GetDisplayIndex on an FName.
#endif
		return FName(GetComparisonIdFromDisplayId(DisplayId), DisplayId, InInternalNumber);
	}

#if UE_FNAME_OUTLINE_NUMBER
	CORE_API static FName FindNumberedName(FNameEntryId DisplayId, int32 InInternalNumber);
#endif //UE_FNAME_OUTLINE_NUMBER

	/**
	 * Default constructor, initialized to None
	 */
	FORCEINLINE constexpr FName()
#if !UE_FNAME_OUTLINE_NUMBER
		: Number(NAME_NO_NUMBER_INTERNAL)
#endif //!UE_FNAME_OUTLINE_NUMBER
	{
	}

	/**
	 * Scary no init constructor, used for something obscure in UObjectBase
	 */
	explicit constexpr FName(ENoInit)
		: ComparisonIndex(NoInit)
#if WITH_CASE_PRESERVING_NAME
		, DisplayIndex(NoInit)
#endif
	{}

	/** Special constructor used by TOptional<FName> */
	explicit FName(FIntrusiveUnsetOptionalState I)
		: ComparisonIndex(I)
#if !UE_FNAME_OUTLINE_NUMBER
		, Number(NAME_NO_NUMBER_INTERNAL)
#endif
#if WITH_CASE_PRESERVING_NAME
		, DisplayIndex(I)
#endif
	{
	}

	FORCEINLINE explicit FName(FMinimalName InName);
	FORCEINLINE explicit FName(FScriptName InName);
	FORCEINLINE FName(FMemoryImageName InName);

	/**
	 * Create an FName. If FindType is FNAME_Find, and the name 
	 * doesn't already exist, then the name will be NAME_None.
	 * The check for existance or not depends on UE_FNAME_OUTLINE_NUMBER.
	 * When UE_FNAME_OUTLINE_NUMBER is 0, we only check for the string part.
	 * When UE_FNAME_OUTLINE_NUMBER is 1, we check for whole name including the number.
	 *
	 * @param Name			Value for the string portion of the name
	 * @param FindType		Action to take (see EFindName, default is FNAME_Add)
	 */
	CORE_API FName(const WIDECHAR* Name);
	CORE_API FName(const ANSICHAR* Name);
	CORE_API FName(const UTF8CHAR* Name);

	CORE_API FName(const WIDECHAR* Name, EFindName FindType);
	CORE_API FName(const ANSICHAR* Name, EFindName FindType);
	CORE_API FName(const UTF8CHAR* Name, EFindName FindType);

	/** Create FName from non-null string with known length  */
	CORE_API FName(int32 Len, const WIDECHAR* Name, EFindName FindType=FNAME_Add);
	CORE_API FName(int32 Len, const ANSICHAR* Name, EFindName FindType=FNAME_Add);
	CORE_API FName(int32 Len, const UTF8CHAR* Name, EFindName FindType=FNAME_Add);

	inline explicit FName(TStringView<ANSICHAR> View, EFindName FindType = FNAME_Add)
		: FName(NoInit)
	{
		*this = FName(View.Len(), View.GetData(), FindType);
	}
	inline explicit FName(TStringView<WIDECHAR> View, EFindName FindType = FNAME_Add)
		: FName(NoInit)
	{
		*this = FName(View.Len(), View.GetData(), FindType);
	}
	inline explicit FName(TStringView<UTF8CHAR> View, EFindName FindType = FNAME_Add)
		: FName(NoInit)
	{
		*this = FName(View.Len(), View.GetData(), FindType);
	}


	/**
	 * Create an FName. Will add the string to the name table if it does not exist.
	 * When UE_FNAME_OUTLINE_NUMBER is set, will also add the combination of base string and number to the name table if it doesn't exist.
	 *
	 * @param Name Value for the string portion of the name
	 * @param InInternalNumber Value for the number portion of the name, in internal format
	 */
	CORE_API FName(const WIDECHAR* Name, int32 InInternalNumber);
	CORE_API FName(const ANSICHAR* Name, int32 InInternalNumber);
	CORE_API FName(const UTF8CHAR* Name, int32 InInternalNumber);
	CORE_API FName(int32 Len, const WIDECHAR* Name, int32 InInternalNumber);
	CORE_API FName(int32 Len, const ANSICHAR* Name, int32 InInternalNumber);
	CORE_API FName(int32 Len, const UTF8CHAR* Name, int32 InInternalNumber);

	inline FName(TStringView<ANSICHAR> View, int32 InInternalNumber)
		: FName(NoInit)
	{
		*this = FName(View.Len(), View.GetData(), InInternalNumber);
	}
	inline FName(TStringView<WIDECHAR> View, int32 InInternalNumber)
		: FName(NoInit)
	{
		*this = FName(View.Len(), View.GetData(), InInternalNumber);
	}
	inline FName(TStringView<UTF8CHAR> View, int32 InInternalNumber)
		: FName(NoInit)
	{
		*this = FName(View.Len(), View.GetData(), InInternalNumber);
	}

	/**
	 * Create an FName.
	 *
	 * @param Name Value for the string portion of the name
	 * @param InInternalNumber Value for the number portion of the name, in internal format
	 * @param bSplitName true if the trailing number should be split from the name when Number == NAME_NO_NUMBER_INTERNAL, or false to always use the name as-is
	 */
	CORE_API FName(const TCHAR* Name, int32 InInternalNumber, bool bSplitName);

	/**
	 * Constructor used by FLinkerLoad when loading its name table; Creates an FName with an instance
	 * number of 0 that does not attempt to split the FName into string and number portions. Also,
	 * this version skips calculating the hashes of the names if possible
	 */
	CORE_API FName(const FNameEntrySerialized& LoadedEntry);

	CORE_API static void DisplayHash( class FOutputDevice& Ar );
	[[nodiscard]] CORE_API static FString SafeString(FNameEntryId InDisplayIndex, int32 InstanceNumber = NAME_NO_NUMBER_INTERNAL);

	CORE_API static void Reserve(uint32 NumBytes, uint32 NumNames);

	/**
	 * @return Size of all name entries.
	 */
	[[nodiscard]] CORE_API static int64 GetNameEntryMemorySize();

	/**
	 * @return Estimated remaining size the name entry table is willing to allocate.
	 */
	[[nodiscard]] CORE_API static int64 GetNameEntryMemoryEstimatedAvailable();

	/**
	* @return Size of Name Table object as a whole
	*/
	[[nodiscard]] CORE_API static int64 GetNameTableMemorySize();

	/**
	 * @return number of ansi names in name table
	 */
	[[nodiscard]] CORE_API static int32 GetNumAnsiNames();

	/**
	 * @return number of wide names in name table
	 */
	[[nodiscard]] CORE_API static int32 GetNumWideNames();

#if UE_FNAME_OUTLINE_NUMBER
	/**
	 * @return number of numbered names in name table
	 */
	[[nodiscard]] CORE_API static int32 GetNumNumberedNames();
#endif

	[[nodiscard]] CORE_API static TArray<const FNameEntry*> DebugDump();

	[[nodiscard]] CORE_API static FNameEntry const* GetEntry(EName Ename);
	[[nodiscard]] CORE_API static FNameEntry const* GetEntry(FNameEntryId Id);

#if UE_TRACE_ENABLED
	[[nodiscard]] CORE_API static UE::Trace::FEventRef32 TraceName(const FName& Name);
	CORE_API static void TraceNamesOnConnection();
#endif

	//@}

	UE_DEPRECATED(5.8, "Tests for FName are registered with both test frameworks now.")
	inline static void AutoTest()
	{
	}
	
	/**
	 * Takes a string and breaks it down into a human readable string.
	 * For example - "bCreateSomeStuff" becomes "Create Some Stuff?" and "DrawScale3D" becomes "Draw Scale 3D".
	 * 
	 * @param	InDisplayName	[In, Out] The name to sanitize
	 * @param	bIsBool				True if the name is a bool
	 *
	 * @return	the sanitized version of the display name
	 */
	[[nodiscard]] CORE_API static FString NameToDisplayString( const FString& InDisplayName, const bool bIsBool );

	/**
	 * Add/remove an exemption to the formatting applied by NameToDisplayString.
	 * Example: exempt the compound word "MetaHuman" to ensure its not reformatted
	 * as "Meta Human".
	 */
	CORE_API static void AddNameToDisplayStringExemption(const FString& InExemption);
	CORE_API static void RemoveNameToDisplayStringExemption(const FString& InExemption);

	/** Get the EName that this FName represents or nullptr */
	[[nodiscard]] CORE_API const EName* ToEName() const;

	/** 
		Tear down system and free all allocated memory 
	
		FName must not be used after teardown
	 */
	CORE_API static void TearDown();

	/** Returns an integer that compares equal in the same way FNames do, only usable within the current process */
#if UE_FNAME_OUTLINE_NUMBER
	[[nodiscard]] FORCEINLINE uint64 ToUnstableInt() const
	{
		return ComparisonIndex.ToUnstableInt();
	}
#else
	[[nodiscard]] FORCEINLINE uint64 ToUnstableInt() const
	{
		static_assert(STRUCT_OFFSET(FName, ComparisonIndex) == 0);
		static_assert(STRUCT_OFFSET(FName, Number) == 4);
		static_assert((STRUCT_OFFSET(FName, Number) + sizeof(Number)) == sizeof(uint64));

		uint64 Out = 0;
		FMemory::Memcpy(&Out, this, sizeof(uint64));
		return Out;
	}
#endif

private:
	/** Index into the Names array (used to find String portion of the string/number pair used for comparison) */
	FNameEntryId	ComparisonIndex;
#if !UE_FNAME_OUTLINE_NUMBER
	/** Number portion of the string/number pair (stored internally as 1 more than actual, so zero'd memory will be the default, no-instance case) */
	uint32			Number;
#endif// ! //UE_FNAME_OUTLINE_NUMBER
#if WITH_CASE_PRESERVING_NAME
	/** Index into the Names array (used to find String portion of the string/number pair used for display) */
	FNameEntryId	DisplayIndex;
#endif // WITH_CASE_PRESERVING_NAME

	friend const TCHAR* DebugFName(int32);
	friend const TCHAR* DebugFName(int32, int32);
	friend const TCHAR* DebugFName(FName&);

	friend struct FNameHelper;
	friend FScriptName NameToScriptName(FName InName);
	friend FMinimalName NameToMinimalName(FName InName);
	friend FMinimalName::FMinimalName(const FName& Name);
	friend FScriptName::FScriptName(const FName& Name);
	friend FMemoryImageName::FMemoryImageName(const FName& Name);

	template <typename StringBufferType>
	FORCEINLINE void AppendStringInternal(StringBufferType& Out) const;

	template <ESearchCase::Type SearchCase>
	FORCEINLINE int32 CompareInternal(const FName& Other) const;

	FORCEINLINE FNameEntryId GetDisplayIndexFast() const
	{
#if WITH_CASE_PRESERVING_NAME
		return DisplayIndex;
#else
		return ComparisonIndex;
#endif
	}

	// Accessor for unmodified comparison index when UE_FNAME_OUTLINE_NUMBER is set
	[[nodiscard]] FORCEINLINE FNameEntryId GetComparisonIndexInternal() const
	{
		return ComparisonIndex;
	}

	// Accessor for unmodified display index when UE_FNAME_OUTLINE_NUMBER is set
	[[nodiscard]] FORCEINLINE FNameEntryId GetDisplayIndexInternal() const
	{
#if WITH_CASE_PRESERVING_NAME
		return DisplayIndex;
#else // WITH_CASE_PRESERVING_NAME
		return ComparisonIndex;
#endif // WITH_CASE_PRESERVING_NAME
	}

	// Resolve the entry directly referred to by LookupId
	[[nodiscard]] CORE_API static const FNameEntry* ResolveEntry(FNameEntryId LookupId);
	// Recursively resolve through the entry referred to by LookupId to reach the allocated string entry, in the case of UE_FNAME_OUTLINE_NUMBER=1
	[[nodiscard]] static const FNameEntry* ResolveEntryRecursive(FNameEntryId LookupId);

	[[nodiscard]] CORE_API static bool IsWithinBounds(FNameEntryId Id);

	// These FNameEntryIds are passed in from user code so they must be non-numbered if Number != NAME_NO_NUMBER_INTERNAL
#if UE_FNAME_OUTLINE_NUMBER
	[[nodiscard]] CORE_API static FName CreateNumberedName(FNameEntryId ComparisonId, FNameEntryId DisplayId, int32 InInternalNumber);
#endif

	[[nodiscard]] FORCEINLINE static FName CreateNumberedNameIfNecessary(FNameEntryId ComparisonId, FNameEntryId DisplayId, int32 InInternalNumber)
	{
#if UE_FNAME_OUTLINE_NUMBER
		if (InInternalNumber != NAME_NO_NUMBER_INTERNAL)
		{
			// We need to store a new entry in the name table
			return CreateNumberedName(ComparisonId, DisplayId, InInternalNumber);
		}

#if DO_CHECK
		if (!IsValidInternalNumber(InInternalNumber))
		{
			ReportInvalidInternalNumber(ComparisonId, InInternalNumber);
		}
#endif

		// Otherwise we can just set the index members
#endif
		FName Out;
		Out.ComparisonIndex = ComparisonId;
#if WITH_CASE_PRESERVING_NAME
		Out.DisplayIndex = DisplayId;
#endif
#if !UE_FNAME_OUTLINE_NUMBER
		Out.Number = InInternalNumber;
#endif
		return Out;
	}

	[[nodiscard]] UE_REWRITE static FName CreateNumberedNameIfNecessary(FNameEntryId ComparisonId, int32 InInternalNumber)
	{
		return CreateNumberedNameIfNecessary(ComparisonId, ComparisonId, InInternalNumber);
	}
	
	
	[[nodiscard]] static bool Equals(FName A, FName B) = delete;
	[[nodiscard]] CORE_API static bool Equals(FName A, FAnsiStringView B);
	[[nodiscard]] CORE_API static bool Equals(FName A, FWideStringView B);
	[[nodiscard]] CORE_API static bool Equals(FName A, const ANSICHAR* B);
	[[nodiscard]] CORE_API static bool Equals(FName A, const WIDECHAR* B);

	[[nodiscard]] FORCEINLINE static bool Equals(FName A, EName B)
	{
		// With UE_FNAME_OUTLINE_NUMBER 1, FName == FName(EName) is
		// faster than extracting index and number for direct EName comparison.
		// return A.GetComparisonIndex() == B && A.GetNumber() == NAME_NO_NUMBER_INTERNAL;
		return A == FName(B);
	}

#if UE_FNAME_OUTLINE_NUMBER
	[[nodiscard]] FORCEINLINE static bool Equals(FName A, FMinimalName B)
	{
		return A.GetComparisonIndexInternal() == B.Index;
	}
	[[nodiscard]] FORCEINLINE static bool Equals(FName A, FScriptName B)
	{
		return A.GetComparisonIndexInternal() == B.ComparisonIndex;
	}
	[[nodiscard]] FORCEINLINE static bool Equals(FName A, FMemoryImageName B)
	{
		return A.GetComparisonIndexInternal() == B.ComparisonIndex;
	}
	[[nodiscard]] friend FORCEINLINE uint32 GetTypeHash(FName Name)
	{
		return GetTypeHash(Name.GetComparisonIndexInternal());
	}
#else
	[[nodiscard]] FORCEINLINE static bool Equals(FName A, FMinimalName B)
	{
		return A.GetComparisonIndex() == B.Index && A.GetNumber() == B.Number;
	}
	[[nodiscard]] FORCEINLINE static bool Equals(FName A, FScriptName B)
	{
		return A.GetComparisonIndex() == B.ComparisonIndex && A.GetNumber() == B.Number;
	}
	[[nodiscard]] FORCEINLINE static bool Equals(FName A, FMemoryImageName B)
	{
		return A.GetComparisonIndex() == B.ComparisonIndex && A.GetNumber() == B.Number;
	}
	[[nodiscard]] friend FORCEINLINE uint32 GetTypeHash(FName Name)
	{
		return GetTypeHashWithNumber(Name.GetComparisonIndex(), Name.GetNumber());
	}
#endif

#if DO_CHECK
	[[noreturn]] CORE_API static void ReportInvalidInternalNumber(FNameEntryId ComparisonId, int32 InternalNumber);
#endif

public:
	template <typename T>
	[[nodiscard]] FORCEINLINE auto UEOpEquals(T Rhs) const -> decltype(FName::Equals(*this, Rhs))
	{
		return FName::Equals(*this, Rhs);
	}
};

template<> struct TIsZeroConstructType<class FName> { enum { Value = true }; };
Expose_TNameOf(FName)

namespace Freeze
{
	// These structures mirror the layout of FMemoryImageName depending on the value of WITH_CASE_PRESERVING_NAME
	// for use in memory image writing/unfreezing
	template<bool bCasePreserving>
	struct TMemoryImageNameLayout;

	template<>
	struct TMemoryImageNameLayout<false>
	{
		FNameEntryId	ComparisonIndex;
		uint32			NumberOrDummy = 0;
	};

	template<>
	struct TMemoryImageNameLayout<true> : public TMemoryImageNameLayout<false>
	{
		FNameEntryId	DisplayIndex;
	};

	CORE_API void ApplyMemoryImageNamePatch(void* NameDst, const FMemoryImageName& Name, const FPlatformTypeLayoutParameters& LayoutParams);

	CORE_API uint32 IntrinsicAppendHash(const FMemoryImageName* DummyObject, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FSHA1& Hasher);
	CORE_API void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const FMemoryImageName& Object, const FTypeLayoutDesc&);
	CORE_API uint32 IntrinsicUnfrozenCopy(const FMemoryUnfreezeContent& Context, const FMemoryImageName& Object, void* OutDst);
	CORE_API void IntrinsicWriteMemoryImage(FMemoryImageWriter& Writer, const FScriptName& Object, const FTypeLayoutDesc&);
}

FORCEINLINE FMemoryImageName::FMemoryImageName()
{
	// The structure must match the layout of Freeze::TMemoryImageNameLayout for the matching value of WITH_CASE_PRESERVING_NAME
	static_assert(sizeof(FMemoryImageName) == sizeof(Freeze::TMemoryImageNameLayout<WITH_CASE_PRESERVING_NAME>));
	static_assert(STRUCT_OFFSET(FMemoryImageName, ComparisonIndex) == STRUCT_OFFSET(Freeze::TMemoryImageNameLayout<WITH_CASE_PRESERVING_NAME>, ComparisonIndex));
#if UE_FNAME_OUTLINE_NUMBER
	static_assert(STRUCT_OFFSET(FMemoryImageName, Dummy) == STRUCT_OFFSET(Freeze::TMemoryImageNameLayout<WITH_CASE_PRESERVING_NAME>, NumberOrDummy));
#else
	static_assert(STRUCT_OFFSET(FMemoryImageName, Number) == STRUCT_OFFSET(Freeze::TMemoryImageNameLayout<WITH_CASE_PRESERVING_NAME>, NumberOrDummy));
#endif
#if WITH_CASE_PRESERVING_NAME
	static_assert(STRUCT_OFFSET(FMemoryImageName, DisplayIndex) == STRUCT_OFFSET(Freeze::TMemoryImageNameLayout<WITH_CASE_PRESERVING_NAME>, DisplayIndex));
#endif

}

FORCEINLINE FMemoryImageName::FMemoryImageName(EName Name)
	: FMemoryImageName(FName(Name))
{
}

DECLARE_INTRINSIC_TYPE_LAYOUT(FMemoryImageName);
DECLARE_INTRINSIC_TYPE_LAYOUT(FScriptName);

#if UE_FNAME_OUTLINE_NUMBER

FORCEINLINE FName::FName(FMinimalName InName)
	: ComparisonIndex(InName.Index)
#if WITH_CASE_PRESERVING_NAME
	, DisplayIndex(InName.Index)
#endif
{
}

FORCEINLINE FName::FName(FScriptName InName)
	: ComparisonIndex(InName.ComparisonIndex)
#if WITH_CASE_PRESERVING_NAME
	, DisplayIndex(InName.DisplayIndex)
#endif
{

}

FORCEINLINE FName::FName(FMemoryImageName InName)
	: ComparisonIndex(InName.ComparisonIndex)
#if WITH_CASE_PRESERVING_NAME
	, DisplayIndex(InName.DisplayIndex)
#endif
{
}

FORCEINLINE FMinimalName::FMinimalName(const FName& Name)
	: Index(Name.GetComparisonIndexInternal())
{
}

FORCEINLINE FScriptName::FScriptName(const FName& Name)
	: ComparisonIndex(Name.GetComparisonIndexInternal())
#if WITH_CASE_PRESERVING_NAME
	, DisplayIndex(Name.GetDisplayIndexInternal())
#endif
{
}

FORCEINLINE FMemoryImageName::FMemoryImageName(const FName& Name)
	: ComparisonIndex(Name.GetComparisonIndexInternal())
#if WITH_CASE_PRESERVING_NAME
	, DisplayIndex(Name.GetDisplayIndexInternal())
#endif
{
}

FORCEINLINE bool FMinimalName::IsNone() const
{
	return Index.IsNone();
}

FORCEINLINE bool FMinimalName::UEOpLessThan(FMinimalName Rhs) const
{
	return Index < Rhs.Index;
}

FORCEINLINE bool FScriptName::IsNone() const
{
	return ComparisonIndex.IsNone();
}

FORCEINLINE bool FMemoryImageName::IsNone() const
{
	return ComparisonIndex.IsNone();
}

FORCEINLINE bool FName::IsEqual(const FName& Rhs, const ENameCase CompareMethod /*= ENameCase::IgnoreCase*/, const bool bCompareNumber /*= true*/) const
{
	return bCompareNumber ?
		(CompareMethod == ENameCase::IgnoreCase ? GetComparisonIndexInternal() == Rhs.GetComparisonIndexInternal() : GetDisplayIndexInternal() == Rhs.GetDisplayIndexInternal()) :  // Unresolved indices include the number, are stored in instances
		(CompareMethod == ENameCase::IgnoreCase ? GetComparisonIndex() == Rhs.GetComparisonIndex() : GetDisplayIndex() == Rhs.GetDisplayIndex()); // Resolved indices, have to hit the name table
}

#else // UE_FNAME_OUTLINE_NUMBER

FORCEINLINE FName::FName(FMinimalName InName)
	: ComparisonIndex(InName.Index)
	, Number(InName.Number)
#if WITH_CASE_PRESERVING_NAME
	, DisplayIndex(InName.Index)
#endif
{
}

FORCEINLINE FName::FName(FScriptName InName)
	: ComparisonIndex(InName.ComparisonIndex)
	, Number(InName.Number)
#if WITH_CASE_PRESERVING_NAME
	, DisplayIndex(InName.DisplayIndex)
#endif
{
}

FORCEINLINE FName::FName(FMemoryImageName InName)
	: ComparisonIndex(InName.ComparisonIndex)
	, Number(InName.Number)
#if WITH_CASE_PRESERVING_NAME
	, DisplayIndex(InName.DisplayIndex)
#endif
{
}

FORCEINLINE FMinimalName::FMinimalName(const FName& Name)
	: Index(Name.GetComparisonIndexInternal())
	, Number(Name.GetNumber())
{
}

FORCEINLINE FScriptName::FScriptName(const FName& Name)
	: ComparisonIndex(Name.GetComparisonIndexInternal())
#if WITH_CASE_PRESERVING_NAME
	, DisplayIndex(Name.GetDisplayIndexInternal())
#endif
	, Number(Name.GetNumber())
{
}

FORCEINLINE FMemoryImageName::FMemoryImageName(const FName& Name)
	: ComparisonIndex(Name.GetComparisonIndex())
	, Number(Name.GetNumber())
#if WITH_CASE_PRESERVING_NAME
	, DisplayIndex(Name.GetDisplayIndex())
#endif
{
}

FORCEINLINE bool FMinimalName::IsNone() const
{
	return Index.IsNone() && Number == NAME_NO_NUMBER_INTERNAL;
}

FORCEINLINE bool FMinimalName::UEOpLessThan(FMinimalName Rhs) const
{
	return Index == Rhs.Index ? Number < Rhs.Number : Index < Rhs.Index;
}

FORCEINLINE bool FScriptName::IsNone() const
{
	return ComparisonIndex.IsNone() && Number == NAME_NO_NUMBER_INTERNAL;
}

FORCEINLINE bool FMemoryImageName::IsNone() const
{
	return ComparisonIndex.IsNone() && Number == NAME_NO_NUMBER_INTERNAL;
}


FORCEINLINE bool FName::IsEqual(const FName& Rhs, const ENameCase CompareMethod /*= ENameCase::IgnoreCase*/, const bool bCompareNumber /*= true*/) const
{
	return ((CompareMethod == ENameCase::IgnoreCase) ? GetComparisonIndex() == Rhs.GetComparisonIndex() : GetDisplayIndexFast() == Rhs.GetDisplayIndexFast())
		&& (!bCompareNumber || GetNumber() == Rhs.GetNumber());
}
#endif // UE_FNAME_OUTLINE_NUMBER

FORCEINLINE FName MinimalNameToName(FMinimalName InName)
{
	return FName(InName);
}

FORCEINLINE FName ScriptNameToName(FScriptName InName)
{
	return FName(InName);
}

FORCEINLINE FMinimalName NameToMinimalName(FName InName)
{
	return FMinimalName(InName);
}

FORCEINLINE FScriptName NameToScriptName(FName InName)
{
	return FScriptName(InName);
}

CORE_API FString LexToString(const FName& Name);

FORCEINLINE void LexFromString(FName& Name, const TCHAR* Str)
{
	Name = FName(Str);
}

inline FWideStringBuilderBase& operator<<(FWideStringBuilderBase& Builder, const FName& Name)
{
	Name.AppendString(Builder);
	return Builder;
}

inline FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Builder, const FName& Name)
{
	Name.AppendString(Builder);
	return Builder;
}

CORE_API FWideStringBuilderBase& operator<<(FWideStringBuilderBase& Builder, FNameEntryId Id);
CORE_API FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Builder, FNameEntryId Id);

/** FNames act like PODs. */
template <> struct TIsPODType<FName> { enum { Value = true }; };

/** Fast non-alphabetical order that is only stable during this process' lifetime */
struct FNameFastLess
{
	FORCEINLINE bool operator()(const FName& A, const FName& B) const
	{
		return A.CompareIndexes(B) < 0;
	}

	FORCEINLINE bool operator()(FNameEntryId A, FNameEntryId B) const
	{
		return A.FastLess(B);
	}
};

/** Slow alphabetical order that is stable / deterministic over process runs */
struct FNameLexicalLess
{
	FORCEINLINE bool operator()(const FName& A, const FName& B) const
	{
		return A.Compare(B) < 0;
	}

	FORCEINLINE bool operator()(FNameEntryId A, FNameEntryId B) const
	{
		return A.LexicalLess(B);
	}
};

struct FNameDebugVisualizer
{
	CORE_API FNameDebugVisualizer(FClangKeepDebugInfo);
	CORE_API uint8** GetBlocks();
private:
	static constexpr uint32 EntryStride = alignof(FNameEntry);
	static constexpr uint32 OffsetBits = 16;
	static constexpr uint32 BlockBits = 13;
	static constexpr uint32 OffsetMask = (1 << OffsetBits) - 1;
	static constexpr uint32 UnusedMask = UINT32_MAX << BlockBits << OffsetBits;
	static constexpr uint32 MaxLength = NAME_SIZE;
};

/** Lazily constructed FName that helps avoid allocating FNames during static initialization */
class FLazyName
{
	// NOTE: Numeric values are used for comparison in UEOpEquals.
	enum class ELiteralType : uint8
	{
		None = 0,
		AnsiLiteral = 1,
		Utf8Literal = 2,
		WideLiteral = 3,
	};

public:
	constexpr FLazyName() = default;

	/** @param Literal must be a string literal */
	template <int N>
	constexpr FLazyName(const ANSICHAR (&Literal)[N])
		: Either{.AnsiLiteral = Literal}
		, Number(ParseNumber(Literal, N - 1))
		, LiteralType(ELiteralType::AnsiLiteral)
	{
	}

	/** @param Literal must be a string literal */
	template <int N>
	constexpr FLazyName(const UTF8CHAR (&Literal)[N])
		: Either{.Utf8Literal = Literal}
		, Number(ParseNumber(Literal, N - 1))
		, LiteralType(ELiteralType::Utf8Literal)
	{
	}

	/** @param Literal must be a string literal */
	template <int N>
	constexpr FLazyName(const WIDECHAR (&Literal)[N])
		: Either{.WideLiteral = Literal}
		, Number(ParseNumber(Literal, N - 1))
		, LiteralType(ELiteralType::WideLiteral)
	{
	}

	explicit FLazyName(FName Name)
		: Either{.PackedName = FLiteralOrName::PackName(Name.GetComparisonIndex(), Name.GetDisplayIndex())}
		, Number(Name.GetNumber())
		, LiteralType(ELiteralType::None)
	{
	}
	
	UE_REWRITE operator FName() const
	{
		return Resolve();
	}

	CORE_API FName Resolve() const;

	CORE_API FString ToString() const;
	CORE_API FUtf8String ToUtf8String() const;

private:
	union FLiteralOrName
	{
		// NOTE: This uses the high bit for the flag. This may be an issue in the future if the high byte
		// of an address starts being used for features like hardware ASan.
		static constexpr uint64 IsNameFlag = uint64(1) << (sizeof(uint64) * 8 - 1);
		static constexpr uint32 DisplayIdShift = 32;

		const ANSICHAR* AnsiLiteral;
		const UTF8CHAR* Utf8Literal;
		const WIDECHAR* WideLiteral;
		mutable uint64 PackedName = 0;

		static constexpr uint64 PackName(FNameEntryId ComparisonId, FNameEntryId DisplayId)
		{
			// FNameEntryId fits in 31 bits -> (DisplayId & IsNameFlag) != 0 -> IsName() == true.
			return IsNameFlag | ComparisonId.ToUnstableInt() |
				(WITH_CASE_PRESERVING_NAME ? (uint64(DisplayId.ToUnstableInt()) << DisplayIdShift) : 0);
		}

		constexpr bool IsName() const
		{
			return !!(PackedName & IsNameFlag);
		}

		inline FNameEntryId GetComparisonId() const
		{
			return FNameEntryId::FromUnstableInt(static_cast<uint32>(PackedName));
		}

		inline FNameEntryId GetDisplayId() const
		{
#if WITH_CASE_PRESERVING_NAME
			return FNameEntryId::FromUnstableInt(static_cast<uint32>((PackedName & ~IsNameFlag) >> DisplayIdShift));
#else
			return GetComparisonId();
#endif
		}
	};

	FLiteralOrName Either;
	uint32 Number = NAME_NO_NUMBER_INTERNAL;
	ELiteralType LiteralType = ELiteralType::None;

	// Parse the number at compile time when possible to avoid runtime initialization overhead.
	// Number must be stored upon construction to allow Resolve() to convert a literal
	// to a name without synchronization. If Number had to be written at the same time
	// it would require synchronization.
	template <typename CharType>
	UE_REWRITE static constexpr uint32 ParseNumber(const CharType* Literal, int32 Len)
	{
		UE_IF_CONSTEVAL
		{
			return UE::Core::Private::ParseNumberFromName(Literal, Len);
		}
		else
		{
			return CallParseNumber(Literal, Len);
		}
	}

	template <typename CharType>
	CORE_API static uint32 CallParseNumber(const CharType* Literal, int32 Len);

public:
	bool UEOpEquals(FName Rhs) const
	{
		// If !Rhs.IsNone(), we have started creating FNames
		// and might as well resolve and cache Lhs
		if (Either.IsName() || !Rhs.IsNone())
		{
			return Rhs == Resolve();
		}
		else if (LiteralType == ELiteralType::AnsiLiteral)
		{
			return Rhs == Either.AnsiLiteral;
		}
		else if (LiteralType == ELiteralType::Utf8Literal)
		{
			return Rhs == Either.Utf8Literal;
		}
		else
		{
			return Rhs == Either.WideLiteral;
		}
	}

	CORE_API bool UEOpEquals(const FLazyName& Rhs) const;

	friend FORCEINLINE uint32 GetTypeHash(FLazyName Name)
	{
		return GetTypeHash(Name.Resolve());
	}
};

// Ordering operators for FName are deliberately disabled as there is no correct default.
// Users should use the following to select the desired behavior:
//
// Lexicographical comparison:
// - Expression: NameA.LexicalLess(NameB)
// - Predicate: FNameLexicalLess
//
// Fast comparison:
// - Expression: NameA.FastLess(NameB)
// - Predicate: FNameFastLess
bool operator<(FName, FName) = delete;
bool operator>(FName, FName) = delete;
bool operator<=(FName, FName) = delete;
bool operator>=(FName, FName) = delete;

/**
 * Serialization util that optimizes WITH_CASE_PRESERVING_NAME-loading by reducing comparison id lookups
 *
 * Stores 32-bit display entry id with an unused bit to indicate if FName::GetComparisonIdFromDisplayId lookup is needed.
 *
 * Note that only display entries should be saved to make output deterministic.
 */
class FDisplayNameEntryId
{
public:
	FDisplayNameEntryId() : FDisplayNameEntryId(FName()) {}
	explicit FDisplayNameEntryId(FName Name) : FDisplayNameEntryId(Name.GetDisplayIndex(), Name.GetComparisonIndex()) {}
	FORCEINLINE FName ToName(uint32 Number) const { return FName(GetComparisonId(), GetDisplayId(), Number); }

private:
#if WITH_CASE_PRESERVING_NAME
	static constexpr uint32 DifferentIdsFlag = 1u << 31;
	static constexpr uint32 DisplayIdMask = ~DifferentIdsFlag;

	uint32 Value = 0;

	FDisplayNameEntryId(FNameEntryId Id, FNameEntryId CmpId) : Value(Id.ToUnstableInt() | (Id != CmpId) * DifferentIdsFlag) {}
	FORCEINLINE bool SameIds() const { return (Value & DifferentIdsFlag) == 0; }
	FORCEINLINE FNameEntryId GetDisplayId() const { return FNameEntryId::FromUnstableInt(Value & DisplayIdMask); }
	FORCEINLINE FNameEntryId GetComparisonId() const { return SameIds() ? GetDisplayId() : FName::GetComparisonIdFromDisplayId(GetDisplayId()); }
public:
	bool UEOpEquals(FDisplayNameEntryId Rhs) const { return Value == Rhs.Value; }
#else
	FNameEntryId Id;

	FDisplayNameEntryId(FNameEntryId InId, FNameEntryId) : Id(InId) {}
	FORCEINLINE FNameEntryId GetDisplayId() const { return Id; }
	FORCEINLINE FNameEntryId GetComparisonId() const { return Id; }
public:
	bool UEOpEquals(FDisplayNameEntryId Rhs) const { return Id == Rhs.Id; }
#endif
	bool UEOpEquals(FNameEntryId Rhs) const { return GetDisplayId() == Rhs; }
	[[nodiscard]] friend uint32 GetTypeHash(FDisplayNameEntryId InId) { return GetTypeHash(InId.GetDisplayId()); }

public: // Internal functions for batch serialization code - intentionally lacking CORE_API
	static FDisplayNameEntryId FromComparisonId(FNameEntryId ComparisonId);
	FNameEntryId ToDisplayId() const;
	void SetLoadedComparisonId(FNameEntryId ComparisonId); // Called first
#if WITH_CASE_PRESERVING_NAME
	void SetLoadedDifferentDisplayId(FNameEntryId DisplayId); // Called second if display id differs
	FNameEntryId GetLoadedComparisonId() const; // Get the already loaded comparison id
#endif
};

/**
 * A string builder with inline storage for FNames.
 */
class FNameBuilder : public TStringBuilder<FName::StringBufferSize>
{
public:
	FNameBuilder() = default;

	inline explicit FNameBuilder(const FName InName)
	{
		InName.AppendString(*this);
	}
};

template <> struct TIsContiguousContainer<FNameBuilder> { static constexpr inline bool Value = true; };

/** Update the Hash with the FName's text and number */
class FBlake3;
CORE_API void AppendHash(FBlake3& Builder, FName In);
