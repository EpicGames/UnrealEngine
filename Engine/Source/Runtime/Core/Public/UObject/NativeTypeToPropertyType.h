// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"

#include "Concepts/UObject.h"
#include "Concepts/UScriptStruct.h"
#include "Concepts/DynamicDelegate.h"
#include "Concepts/DynamicMulticastDelegate.h"
#include "Concepts/SparseDynamicMulticastDelegate.h"
#include "Concepts/UEnum.h"
#include "Containers/ContainersFwd.h"
#include "Misc/OptionalFwd.h"
#include "UObject/UObjectHierarchyFwd.h"
#include "UObject/WeakObjectPtrTemplatesFwd.h"

class FAnsiString;
struct FFieldPath;
struct FLazyObjectPtr;
class FName;
struct FObjectPtr;
class FScriptInterface;
struct FSoftObjectPtr;
struct FWeakObjectPtr;
class FString;
class FText;
class FUtf8String;

template <typename InEnumType> class TEnumAsByte;
template <typename T> struct TFieldPath;
template <typename T> struct TLazyObjectPtr;
template <typename T> struct TObjectPtr;
template <typename T> class TScriptInterface;
template <typename T> class TSoftClassPtr;
template <typename T> struct TSoftObjectPtr;
template <typename MulticastDelegate, typename OwningClass, typename DelegateInfoClass> struct TSparseDynamicDelegate;
template <typename T> class TSubclassOf;


// This trait defines how different native arguments map to property types.
//
// Using this trait isn't sufficient to fully create an instance of a property class given a native type, as some types
// will require inner setup too, e.g. a property representing TOptional<int32> requires creating an FOptionalProperty
// with an inner FIntProperty, or a property representing FVector will need an inner UStruct*.  Use a helper function
// like CreatePropertyForNativeType<T>() for a complete property.
//
// There is overlap with TCPPTypeToPropertyType in Editor/Kismet/Internal/Reflections/FunctionUtilsPrivate.h, so
// these could be unified in future.
template <typename T>
struct TNativeTypeToPropertyType
{
};

template <>                                         struct TNativeTypeToPropertyType<uint8>                                 { using Type = FByteProperty; };
template <typename InEnumType>                      struct TNativeTypeToPropertyType<TEnumAsByte<InEnumType>>               { using Type = FByteProperty; };
template <>                                         struct TNativeTypeToPropertyType<uint16>                                { using Type = FUInt16Property; };
template <>                                         struct TNativeTypeToPropertyType<uint32>                                { using Type = FUInt32Property; };
template <>                                         struct TNativeTypeToPropertyType<uint64>                                { using Type = FUInt64Property; };
template <>                                         struct TNativeTypeToPropertyType<int8>                                  { using Type = FInt8Property; };
template <>                                         struct TNativeTypeToPropertyType<int16>                                 { using Type = FInt16Property; };
template <>                                         struct TNativeTypeToPropertyType<int32>                                 { using Type = FIntProperty; };
template <>                                         struct TNativeTypeToPropertyType<int64>                                 { using Type = FInt64Property; };
template <>                                         struct TNativeTypeToPropertyType<bool>                                  { using Type = FBoolProperty; };
template <>                                         struct TNativeTypeToPropertyType<float>                                 { using Type = FFloatProperty; };
template <>                                         struct TNativeTypeToPropertyType<double>                                { using Type = FDoubleProperty; };
template <UE::CUObject T>                           struct TNativeTypeToPropertyType<T*>                                    { using Type = std::conditional_t<std::is_base_of_v<UClass, T>, FClassProperty, FObjectProperty>; };
template <>                                         struct TNativeTypeToPropertyType<FObjectPtr>                            { using Type = FObjectProperty; };
template <UE::CUObject T>                           struct TNativeTypeToPropertyType<TObjectPtr<T>>                         { using Type = typename TNativeTypeToPropertyType<T*>::Type; };
template <typename T>                               struct TNativeTypeToPropertyType<TSubclassOf<T>>                        { using Type = FClassProperty; };
template <>                                         struct TNativeTypeToPropertyType<FScriptInterface>                      { using Type = FInterfaceProperty; };
template <typename T>                               struct TNativeTypeToPropertyType<TScriptInterface<T>>                   { using Type = FInterfaceProperty; };
template <>                                         struct TNativeTypeToPropertyType<FWeakObjectPtr>                        { using Type = FWeakObjectProperty; };
template <typename T>                               struct TNativeTypeToPropertyType<TWeakObjectPtr<T>>                     { using Type = FWeakObjectProperty; };
template <>                                         struct TNativeTypeToPropertyType<FLazyObjectPtr>                        { using Type = FLazyObjectProperty; };
template <typename T>                               struct TNativeTypeToPropertyType<TLazyObjectPtr<T>>                     { using Type = FLazyObjectProperty; };
template <>                                         struct TNativeTypeToPropertyType<FSoftObjectPtr>                        { using Type = FSoftObjectProperty; };
template <typename T>                               struct TNativeTypeToPropertyType<TSoftObjectPtr<T>>                     { using Type = FSoftObjectProperty; };
template <typename T>                               struct TNativeTypeToPropertyType<TSoftClassPtr<T>>                      { using Type = FSoftClassProperty; };
template <>                                         struct TNativeTypeToPropertyType<FName>                                 { using Type = FNameProperty; };
template <UE::CUScriptStruct T>                     struct TNativeTypeToPropertyType<T>                                     { using Type = FStructProperty; };
template <>                                         struct TNativeTypeToPropertyType<FString>                               { using Type = FStrProperty; };
template <>                                         struct TNativeTypeToPropertyType<FText>                                 { using Type = FTextProperty; };
template <typename T>                               struct TNativeTypeToPropertyType<TArray<T, FDefaultAllocator>>          { using Type = FArrayProperty; };
template <UE::CDynamicDelegate T>                   struct TNativeTypeToPropertyType<T>                                     { using Type = FDelegateProperty; };
template <UE::CDynamicMulticastDelegate T>          struct TNativeTypeToPropertyType<T>                                     { using Type = FMulticastInlineDelegateProperty; };
template <UE::CSparseDynamicMulticastDelegate T>    struct TNativeTypeToPropertyType<T>                                     { using Type = FMulticastSparseDelegateProperty; };
template <typename InKeyType, typename InValueType> struct TNativeTypeToPropertyType<TMap<InKeyType, InValueType>>          { using Type = FMapProperty; };
template <typename InElementType>                   struct TNativeTypeToPropertyType<TSet<InElementType>>                   { using Type = FSetProperty; };
template <UE::CUEnum T>                             struct TNativeTypeToPropertyType<T>                                     { using Type = FEnumProperty; };
template <>                                         struct TNativeTypeToPropertyType<FUtf8String>                           { using Type = FUtf8StrProperty; };
template <>                                         struct TNativeTypeToPropertyType<FAnsiString>                           { using Type = FAnsiStrProperty; };
template <typename T>                               struct TNativeTypeToPropertyType<TOptional<T>>                          { using Type = FOptionalProperty; };
template <>                                         struct TNativeTypeToPropertyType<FFieldPath>                            { using Type = FFieldPathProperty; };
template <typename T>                               struct TNativeTypeToPropertyType<TFieldPath<T>>                         { using Type = FFieldPathProperty; };

template <typename T> struct TNativeTypeToPropertyType<const          T> : TNativeTypeToPropertyType<T> {};
template <typename T> struct TNativeTypeToPropertyType<      volatile T> : TNativeTypeToPropertyType<T> {};
template <typename T> struct TNativeTypeToPropertyType<const volatile T> : TNativeTypeToPropertyType<T> {};

template <typename T>
using TNativeTypeToPropertyType_T = typename TNativeTypeToPropertyType<T>::Type;
