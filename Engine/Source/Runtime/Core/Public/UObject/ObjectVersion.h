// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class FArchive;
class FCbFieldView;
class FCbObject;
class FCbWriter;

// Prevents incorrect files from being loaded.

#define PACKAGE_FILE_TAG			0x9E2A83C1
#define PACKAGE_FILE_TAG_SWAPPED	0xC1832A9E

// Cast to ensure that the construct cannot be used in a #if without compiler error.
// This is useful as enum vales cannot be seen by the preprocessor.
#define PREPROCESSOR_ENUM_PROTECT(a) ((unsigned int)(a))

/**
 * Versioning
 *
 * We now have two enums for global versioning, EUnrealEngineObjectUE5Version and EUnrealEngineObjectUE4Version
 * which can be updated independently of each other. This allows us to potentially make version changed in the UE4
 * code base in the future and still have packages saved with those changes to be compatible with the UE5 code base.
 * 
 * The struct FPackageFileVersion should be used to contain the version number rather than storing the UE4/UE5 specific
 * version numbers. It provides overloads for testing against both version enum types for convenience.
 *
 * Note that when reporting versioning problems we often use FPackageFileVersion::ToValue which returns the highest
 * version valid in the structure. This allows us to keep reporting code simple and hide the fact that there are two
 * version numbers from end users. However this does mean that if a version mismatch is encountered due to a future
 * extension of EUnrealEngineObjectUE4Version, then the error message given might look odd. Although given that we are
 * unlikely to be changing EUnrealEngineObjectUE4Version in the future the risk is deemed worth it.
 */


/** This enum is the version for UE5 and should ONLY be edited in UE5Main branch! */
enum class EUnrealEngineObjectUE5Version : uint32
{
	// Note that currently the oldest loadable package version is EUnrealEngineObjectUEVersion::VER_UE4_OLDEST_LOADABLE_PACKAGE
	// this can be enabled should we ever deprecate UE4 versions entirely
	//OLDEST_LOADABLE_PACKAGE = ???,

	// The original UE5 version, at the time this was added the UE4 version was 522, so UE5 will start from 1000 to show a clear difference
	INITIAL_VERSION = 1000,

	// Support stripping names that are not referenced from export data
	NAMES_REFERENCED_FROM_EXPORT_DATA,

	// Added a payload table of contents to the package summary
	PAYLOAD_TOC,

	// Added data to identify references from and to optional package
	OPTIONAL_RESOURCES,

	// Large world coordinates converts a number of core types to double components by default.
	LARGE_WORLD_COORDINATES,

	// Remove package GUID from FObjectExport
	REMOVE_OBJECT_EXPORT_PACKAGE_GUID,

	// Add IsInherited to the FObjectExport entry
	TRACK_OBJECT_EXPORT_IS_INHERITED,

	// Replace FName asset path in FSoftObjectPath with (package name, asset name) pair FTopLevelAssetPath
	FSOFTOBJECTPATH_REMOVE_ASSET_PATH_FNAMES,

	// Add a soft object path list to the package summary for fast remap
	ADD_SOFTOBJECTPATH_LIST,

	// Added bulk/data resource table
	DATA_RESOURCES,

	// Added script property serialization offset to export table entries for saved, versioned packages
	SCRIPT_SERIALIZATION_OFFSET,

	// Adding property tag extension,
	// Support for overridable serialization on UObject,
	// Support for overridable logic in containers
	PROPERTY_TAG_EXTENSION_AND_OVERRIDABLE_SERIALIZATION,

	// Added property tag complete type name and serialization type
	PROPERTY_TAG_COMPLETE_TYPE_NAME,

	// Changed UE::AssetRegistry::WritePackageData to include PackageBuildDependencies
	ASSETREGISTRY_PACKAGEBUILDDEPENDENCIES,

	// Added meta data serialization offset to for saved, versioned packages
	METADATA_SERIALIZATION_OFFSET,

	// Added VCells to the object graph
	VERSE_CELLS,

	// Changed PackageFileSummary to write FIoHash PackageSavedHash instead of FGuid Guid
	PACKAGE_SAVED_HASH,

	// OS shadow serialization of subobjects
	OS_SUB_OBJECT_SHADOW_SERIALIZATION,

	// Adds a table of hierarchical type information for imports in a package
	IMPORT_TYPE_HIERARCHIES,

	// -----<new versions can be added before this line>-------------------------------------------------
	// - this needs to be the last line (see note below)
	AUTOMATIC_VERSION_PLUS_ONE,
	AUTOMATIC_VERSION = AUTOMATIC_VERSION_PLUS_ONE - 1
};

/** This enum is the version for UE4 and should ONLY be edited in UE4 Main branch! */
enum EUnrealEngineObjectUE4Version
{
	VER_UE4_OLDEST_LOADABLE_PACKAGE = 214,

	// Removed restriction on blueprint-exposed variables from being read-only
	VER_UE4_BLUEPRINT_VARS_NOT_READ_ONLY,
	// Added manually serialized element to UStaticMesh (precalculated nav collision)
	VER_UE4_STATIC_MESH_STORE_NAV_COLLISION,
	// Changed property name for atmospheric fog
	VER_UE4_ATMOSPHERIC_FOG_DECAY_NAME_CHANGE,
	// Change many properties/functions from Translation to Location
	VER_UE4_SCENECOMP_TRANSLATION_TO_LOCATION,
	// Material attributes reordering
	VER_UE4_MATERIAL_ATTRIBUTES_REORDERING,
	// Collision Profile setting has been added, and all components that exists has to be properly upgraded
	VER_UE4_COLLISION_PROFILE_SETTING,
	// Making the blueprint's skeleton class transient
	VER_UE4_BLUEPRINT_SKEL_TEMPORARY_TRANSIENT,
	// Making the blueprint's skeleton class serialized again
	VER_UE4_BLUEPRINT_SKEL_SERIALIZED_AGAIN,
	// Blueprint now controls replication settings again
	VER_UE4_BLUEPRINT_SETS_REPLICATION,
	// Added level info used by World browser
	VER_UE4_WORLD_LEVEL_INFO,
	// Changed capsule height to capsule half-height (afterwards)
	VER_UE4_AFTER_CAPSULE_HALF_HEIGHT_CHANGE,
	// Added Namepace, GUID (Key) and Flags to FText
	VER_UE4_ADDED_NAMESPACE_AND_KEY_DATA_TO_FTEXT,
	// Attenuation shapes
	VER_UE4_ATTENUATION_SHAPES,
	// Use IES texture multiplier even when IES brightness is not being used
	VER_UE4_LIGHTCOMPONENT_USE_IES_TEXTURE_MULTIPLIER_ON_NON_IES_BRIGHTNESS,
	// Removed InputComponent as a blueprint addable component
	VER_UE4_REMOVE_INPUT_COMPONENTS_FROM_BLUEPRINTS,
	// Use an FMemberReference struct in UK2Node_Variable
	VER_UE4_VARK2NODE_USE_MEMBERREFSTRUCT,
	// Refactored material expression inputs for UMaterialExpressionSceneColor and UMaterialExpressionSceneDepth
	VER_UE4_REFACTOR_MATERIAL_EXPRESSION_SCENECOLOR_AND_SCENEDEPTH_INPUTS,
	// Spline meshes changed from Z forwards to configurable
	VER_UE4_SPLINE_MESH_ORIENTATION,
	// Added ReverbEffect asset type
	VER_UE4_REVERB_EFFECT_ASSET_TYPE,
	// changed max texcoords from 4 to 8
	VER_UE4_MAX_TEXCOORD_INCREASED,
	// static meshes changed to support SpeedTrees
	VER_UE4_SPEEDTREE_STATICMESH,
	// Landscape component reference between landscape component and collision component
	VER_UE4_LANDSCAPE_COMPONENT_LAZY_REFERENCES,
	// Refactored UK2Node_CallFunction to use FMemberReference
	VER_UE4_SWITCH_CALL_NODE_TO_USE_MEMBER_REFERENCE,
	// Added fixup step to remove skeleton class references from blueprint objects
	VER_UE4_ADDED_SKELETON_ARCHIVER_REMOVAL,
	// See above, take 2.
	VER_UE4_ADDED_SKELETON_ARCHIVER_REMOVAL_SECOND_TIME,
	// Making the skeleton class on blueprints transient
	VER_UE4_BLUEPRINT_SKEL_CLASS_TRANSIENT_AGAIN,
	// UClass knows if it's been cooked
	VER_UE4_ADD_COOKED_TO_UCLASS,
	// Deprecated static mesh thumbnail properties were removed
	VER_UE4_DEPRECATED_STATIC_MESH_THUMBNAIL_PROPERTIES_REMOVED,
	// Added collections in material shader map ids
	VER_UE4_COLLECTIONS_IN_SHADERMAPID,
	// Renamed some Movement Component properties, added PawnMovementComponent
	VER_UE4_REFACTOR_MOVEMENT_COMPONENT_HIERARCHY,
	// Swap UMaterialExpressionTerrainLayerSwitch::LayerUsed/LayerNotUsed the correct way round
	VER_UE4_FIX_TERRAIN_LAYER_SWITCH_ORDER,
	// Remove URB_ConstraintSetup
	VER_UE4_ALL_PROPS_TO_CONSTRAINTINSTANCE,
	// Low quality directional lightmaps
	VER_UE4_LOW_QUALITY_DIRECTIONAL_LIGHTMAPS,
	// Added NoiseEmitterComponent and removed related Pawn properties.
	VER_UE4_ADDED_NOISE_EMITTER_COMPONENT,
	// Add text component vertical alignment
	VER_UE4_ADD_TEXT_COMPONENT_VERTICAL_ALIGNMENT,
	// Added AssetImportData for FBX asset types, deprecating SourceFilePath and SourceFileTimestamp
	VER_UE4_ADDED_FBX_ASSET_IMPORT_DATA,
	// Remove LevelBodySetup from ULevel
	VER_UE4_REMOVE_LEVELBODYSETUP,
	// Refactor character crouching
	VER_UE4_REFACTOR_CHARACTER_CROUCH,
	// Trimmed down material shader debug information.
	VER_UE4_SMALLER_DEBUG_MATERIALSHADER_UNIFORM_EXPRESSIONS,
	// APEX Clothing
	VER_UE4_APEX_CLOTH,
	// Change Collision Channel to save only modified ones than all of them
	// @note!!! once we pass this CL, we can rename FCollisionResponseContainer enum values
	// we should rename to match ECollisionChannel
	VER_UE4_SAVE_COLLISIONRESPONSE_PER_CHANNEL,
	// Added Landscape Spline editor meshes
	VER_UE4_ADDED_LANDSCAPE_SPLINE_EDITOR_MESH,
	// Fixup input expressions for reading from refraction material attributes.
	VER_UE4_CHANGED_MATERIAL_REFACTION_TYPE,
	// Refactor projectile movement, along with some other movement component work.
	VER_UE4_REFACTOR_PROJECTILE_MOVEMENT,
	// Remove PhysicalMaterialProperty and replace with user defined enum
	VER_UE4_REMOVE_PHYSICALMATERIALPROPERTY,
	// Removed all compile outputs from FMaterial
	VER_UE4_PURGED_FMATERIAL_COMPILE_OUTPUTS,
	// Ability to save cooked PhysX meshes to Landscape
	VER_UE4_ADD_COOKED_TO_LANDSCAPE,
	// Change how input component consumption works
	VER_UE4_CONSUME_INPUT_PER_BIND,
	// Added new Graph based SoundClass Editor
	VER_UE4_SOUND_CLASS_GRAPH_EDITOR,
	// Fixed terrain layer node guids which was causing artifacts
	VER_UE4_FIXUP_TERRAIN_LAYER_NODES,
	// Added clamp min/max swap check to catch older materials
	VER_UE4_RETROFIT_CLAMP_EXPRESSIONS_SWAP,
	// Remove static/movable/stationary light classes
	VER_UE4_REMOVE_LIGHT_MOBILITY_CLASSES,
	// Refactor the way physics blending works to allow partial blending
	VER_UE4_REFACTOR_PHYSICS_BLENDING,
	// WorldLevelInfo: Added reference to parent level and streaming distance
	VER_UE4_WORLD_LEVEL_INFO_UPDATED,
	// Fixed cooking of skeletal/static meshes due to bad serialization logic
	VER_UE4_STATIC_SKELETAL_MESH_SERIALIZATION_FIX,
	// Removal of InterpActor and PhysicsActor
	VER_UE4_REMOVE_STATICMESH_MOBILITY_CLASSES,
	// Refactor physics transforms
	VER_UE4_REFACTOR_PHYSICS_TRANSFORMS,
	// Remove zero triangle sections from static meshes and compact material indices.
	VER_UE4_REMOVE_ZERO_TRIANGLE_SECTIONS,
	// Add param for deceleration in character movement instead of using acceleration.
	VER_UE4_CHARACTER_MOVEMENT_DECELERATION,
	// Made ACameraActor use a UCameraComponent for parameter storage, etc...
	VER_UE4_CAMERA_ACTOR_USING_CAMERA_COMPONENT,
	// Deprecated some pitch/roll properties in CharacterMovementComponent
	VER_UE4_CHARACTER_MOVEMENT_DEPRECATE_PITCH_ROLL,
	// Rebuild texture streaming data on load for uncooked builds
	VER_UE4_REBUILD_TEXTURE_STREAMING_DATA_ON_LOAD,
	// Add support for 32 bit index buffers for static meshes.
	VER_UE4_SUPPORT_32BIT_STATIC_MESH_INDICES,
	// Added streaming install ChunkID to AssetData and UPackage
	VER_UE4_ADDED_CHUNKID_TO_ASSETDATA_AND_UPACKAGE,
	// Add flag to control whether Character blueprints receive default movement bindings.
	VER_UE4_CHARACTER_DEFAULT_MOVEMENT_BINDINGS,
	// APEX Clothing LOD Info
	VER_UE4_APEX_CLOTH_LOD,
	// Added atmospheric fog texture data to be general
	VER_UE4_ATMOSPHERIC_FOG_CACHE_DATA,
	// Arrays serialize their inner's tags
	VAR_UE4_ARRAY_PROPERTY_INNER_TAGS,
	// Skeletal mesh index data is kept in memory in game to support mesh merging.
	VER_UE4_KEEP_SKEL_MESH_INDEX_DATA,
	// Added compatibility for the body instance collision change
	VER_UE4_BODYSETUP_COLLISION_CONVERSION,
	// Reflection capture cooking
	VER_UE4_REFLECTION_CAPTURE_COOKING,
	// Removal of DynamicTriggerVolume, DynamicBlockingVolume, DynamicPhysicsVolume
	VER_UE4_REMOVE_DYNAMIC_VOLUME_CLASSES,
	// Store an additional flag in the BodySetup to indicate whether there is any cooked data to load
	VER_UE4_STORE_HASCOOKEDDATA_FOR_BODYSETUP,
	// Changed name of RefractionBias to RefractionDepthBias.
	VER_UE4_REFRACTION_BIAS_TO_REFRACTION_DEPTH_BIAS,
	// Removal of SkeletalPhysicsActor
	VER_UE4_REMOVE_SKELETALPHYSICSACTOR,
	// PlayerController rotation input refactor
	VER_UE4_PC_ROTATION_INPUT_REFACTOR,
	// Landscape Platform Data cooking
	VER_UE4_LANDSCAPE_PLATFORMDATA_COOKING,
	// Added call for linking classes in CreateExport to ensure memory is initialized properly
	VER_UE4_CREATEEXPORTS_CLASS_LINKING_FOR_BLUEPRINTS,
	// Remove native component nodes from the blueprint SimpleConstructionScript
	VER_UE4_REMOVE_NATIVE_COMPONENTS_FROM_BLUEPRINT_SCS,
	// Removal of Single Node Instance
	VER_UE4_REMOVE_SINGLENODEINSTANCE, 
	// Character movement braking changes
	VER_UE4_CHARACTER_BRAKING_REFACTOR,
	// Supported low quality lightmaps in volume samples
	VER_UE4_VOLUME_SAMPLE_LOW_QUALITY_SUPPORT,
	// Split bEnableTouchEvents out from bEnableClickEvents
	VER_UE4_SPLIT_TOUCH_AND_CLICK_ENABLES,
	// Health/Death refactor
	VER_UE4_HEALTH_DEATH_REFACTOR,
	// Moving USoundNodeEnveloper from UDistributionFloatConstantCurve to FRichCurve
	VER_UE4_SOUND_NODE_ENVELOPER_CURVE_CHANGE,
	// Moved SourceRadius to UPointLightComponent
	VER_UE4_POINT_LIGHT_SOURCE_RADIUS,
	// Scene capture actors based on camera actors.
	VER_UE4_SCENE_CAPTURE_CAMERA_CHANGE,
	// Moving SkeletalMesh shadow casting flag from LoD details to material
	VER_UE4_MOVE_SKELETALMESH_SHADOWCASTING,
	// Changing bytecode operators for creating arrays
	VER_UE4_CHANGE_SETARRAY_BYTECODE,
	// Material Instances overriding base material properties.
	VER_UE4_MATERIAL_INSTANCE_BASE_PROPERTY_OVERRIDES,
	// Combined top/bottom lightmap textures
	VER_UE4_COMBINED_LIGHTMAP_TEXTURES,
	// Forced material lightmass guids to be regenerated
	VER_UE4_BUMPED_MATERIAL_EXPORT_GUIDS,
	// Allow overriding of parent class input bindings
	VER_UE4_BLUEPRINT_INPUT_BINDING_OVERRIDES,
	// Fix up convex invalid transform
	VER_UE4_FIXUP_BODYSETUP_INVALID_CONVEX_TRANSFORM, 
	// Fix up scale of physics stiffness and damping value
	VER_UE4_FIXUP_STIFFNESS_AND_DAMPING_SCALE, 
	// Convert USkeleton and FBoneContrainer to using FReferenceSkeleton.
	VER_UE4_REFERENCE_SKELETON_REFACTOR,
	// Adding references to variable, function, and macro nodes to be able to update to renamed values
	VER_UE4_K2NODE_REFERENCEGUIDS,
	// Fix up the 0th bone's parent bone index.
	VER_UE4_FIXUP_ROOTBONE_PARENT,
	//Allow setting of TextRenderComponents size in world space.
	VER_UE4_TEXT_RENDER_COMPONENTS_WORLD_SPACE_SIZING,
	// Material Instances overriding base material properties #2.
	VER_UE4_MATERIAL_INSTANCE_BASE_PROPERTY_OVERRIDES_PHASE_2,
	// CLASS_Placeable becomes CLASS_NotPlaceable
	VER_UE4_CLASS_NOTPLACEABLE_ADDED,
	// Added LOD info list to a world tile description
	VER_UE4_WORLD_LEVEL_INFO_LOD_LIST,
	// CharacterMovement variable naming refactor
	VER_UE4_CHARACTER_MOVEMENT_VARIABLE_RENAMING_1,
	// FName properties containing sound names converted to FSlateSound properties
	VER_UE4_FSLATESOUND_CONVERSION,
	// Added ZOrder to a world tile description
	VER_UE4_WORLD_LEVEL_INFO_ZORDER,
	// Added flagging of localization gather requirement to packages
	VER_UE4_PACKAGE_REQUIRES_LOCALIZATION_GATHER_FLAGGING,
	// Preventing Blueprint Actor variables from having default values
	VER_UE4_BP_ACTOR_VARIABLE_DEFAULT_PREVENTING,
	// Preventing Blueprint Actor variables from having default values
	VER_UE4_TEST_ANIMCOMP_CHANGE,
	// Class as primary asset, name convention changed
	VER_UE4_EDITORONLY_BLUEPRINTS,
	// Custom serialization for FEdGraphPinType
	VER_UE4_EDGRAPHPINTYPE_SERIALIZATION,
	// Stop generating 'mirrored' cooked mesh for Brush and Model components
	VER_UE4_NO_MIRROR_BRUSH_MODEL_COLLISION,
	// Changed ChunkID to be an array of IDs.
	VER_UE4_CHANGED_CHUNKID_TO_BE_AN_ARRAY_OF_CHUNKIDS,
	// Worlds have been renamed from "TheWorld" to be named after the package containing them
	VER_UE4_WORLD_NAMED_AFTER_PACKAGE,
	// Added sky light component
	VER_UE4_SKY_LIGHT_COMPONENT,
	// Added Enable distance streaming flag to FWorldTileLayer
	VER_UE4_WORLD_LAYER_ENABLE_DISTANCE_STREAMING,
	// Remove visibility/zone information from UModel
	VER_UE4_REMOVE_ZONES_FROM_MODEL,
	// Fix base pose serialization 
	VER_UE4_FIX_ANIMATIONBASEPOSE_SERIALIZATION,
	// Support for up to 8 skinning influences per vertex on skeletal meshes (on non-gpu vertices)
	VER_UE4_SUPPORT_8_BONE_INFLUENCES_SKELETAL_MESHES,
	// Add explicit bOverrideGravity to world settings
	VER_UE4_ADD_OVERRIDE_GRAVITY_FLAG,
	// Support for up to 8 skinning influences per vertex on skeletal meshes (on gpu vertices)
	VER_UE4_SUPPORT_GPUSKINNING_8_BONE_INFLUENCES,
	// Supporting nonuniform scale animation
	VER_UE4_ANIM_SUPPORT_NONUNIFORM_SCALE_ANIMATION,
	// Engine version is stored as a FEngineVersion object rather than changelist number
	VER_UE4_ENGINE_VERSION_OBJECT,
	// World assets now have RF_Public
	VER_UE4_PUBLIC_WORLDS,
	// Skeleton Guid
	VER_UE4_SKELETON_GUID_SERIALIZATION,
	// Character movement WalkableFloor refactor
	VER_UE4_CHARACTER_MOVEMENT_WALKABLE_FLOOR_REFACTOR,
	// Lights default to inverse squared
	VER_UE4_INVERSE_SQUARED_LIGHTS_DEFAULT,
	// Disabled SCRIPT_LIMIT_BYTECODE_TO_64KB
	VER_UE4_DISABLED_SCRIPT_LIMIT_BYTECODE,
	// Made remote role private, exposed bReplicates
	VER_UE4_PRIVATE_REMOTE_ROLE,
	// Fix up old foliage components to have static mobility (superseded by VER_UE4_FOLIAGE_MOVABLE_MOBILITY)
	VER_UE4_FOLIAGE_STATIC_MOBILITY,
	// Change BuildScale from a float to a vector
	VER_UE4_BUILD_SCALE_VECTOR,
	// After implementing foliage collision, need to disable collision on old foliage instances
	VER_UE4_FOLIAGE_COLLISION,
	// Added sky bent normal to indirect lighting cache
	VER_UE4_SKY_BENT_NORMAL,
	// Added cooking for landscape collision data
	VER_UE4_LANDSCAPE_COLLISION_DATA_COOKING,
	// Convert CPU tangent Z delta to vector from PackedNormal since we don't get any benefit other than memory
	// we still convert all to FVector in CPU time whenever any calculation
	VER_UE4_MORPHTARGET_CPU_TANGENTZDELTA_FORMATCHANGE,
	// Soft constraint limits will implicitly use the mass of the bodies
	VER_UE4_SOFT_CONSTRAINTS_USE_MASS,
	// Reflection capture data saved in packages
	VER_UE4_REFLECTION_DATA_IN_PACKAGES,
	// Fix up old foliage components to have movable mobility (superseded by VER_UE4_FOLIAGE_STATIC_LIGHTING_SUPPORT)
	VER_UE4_FOLIAGE_MOVABLE_MOBILITY,
	// Undo BreakMaterialAttributes changes as it broke old content
	VER_UE4_UNDO_BREAK_MATERIALATTRIBUTES_CHANGE,
	// Now Default custom profile name isn't NONE anymore due to copy/paste not working properly with it
	VER_UE4_ADD_CUSTOMPROFILENAME_CHANGE, 
	// Permanently flip and scale material expression coordinates
	VER_UE4_FLIP_MATERIAL_COORDS,
	// PinSubCategoryMemberReference added to FEdGraphPinType
	VER_UE4_MEMBERREFERENCE_IN_PINTYPE,
	// Vehicles use Nm for Torque instead of cm and RPM instead of rad/s
	VER_UE4_VEHICLES_UNIT_CHANGE,
	// removes NANs from all animations when loaded
	// now importing should detect NaNs, so we should not have NaNs in source data
	VER_UE4_ANIMATION_REMOVE_NANS,
	// Change skeleton preview attached assets property type
	VER_UE4_SKELETON_ASSET_PROPERTY_TYPE_CHANGE,
	// Fix some blueprint variables that have the CPF_DisableEditOnTemplate flag set
	// when they shouldn't
	VER_UE4_FIX_BLUEPRINT_VARIABLE_FLAGS,
	// Vehicles use Nm for Torque instead of cm and RPM instead of rad/s part two (missed conversion for some variables
	VER_UE4_VEHICLES_UNIT_CHANGE2,
	// Changed order of interface class serialization
	VER_UE4_UCLASS_SERIALIZE_INTERFACES_AFTER_LINKING,
	// Change from LOD distances to display factors
	VER_UE4_STATIC_MESH_SCREEN_SIZE_LODS,
	// Requires test of material coords to ensure they're saved correctly
	VER_UE4_FIX_MATERIAL_COORDS,
	// Changed SpeedTree wind presets to v7
	VER_UE4_SPEEDTREE_WIND_V7,
	// NeedsLoadForEditorGame added
	VER_UE4_LOAD_FOR_EDITOR_GAME,
	// Manual serialization of FRichCurveKey to save space
	VER_UE4_SERIALIZE_RICH_CURVE_KEY,
	// Change the outer of ULandscapeMaterialInstanceConstants and Landscape-related textures to the level in which they reside
	VER_UE4_MOVE_LANDSCAPE_MICS_AND_TEXTURES_WITHIN_LEVEL,
	// FTexts have creation history data, removed Key, Namespaces, and SourceString
	VER_UE4_FTEXT_HISTORY,
	// Shift comments to the left to contain expressions properly
	VER_UE4_FIX_MATERIAL_COMMENTS,
	// Bone names stored as FName means that we can't guarantee the correct case on export, now we store a separate string for export purposes only
	VER_UE4_STORE_BONE_EXPORT_NAMES,
	// changed mesh emitter initial orientation to distribution
	VER_UE4_MESH_EMITTER_INITIAL_ORIENTATION_DISTRIBUTION,
	// Foliage on blueprints causes crashes
	VER_UE4_DISALLOW_FOLIAGE_ON_BLUEPRINTS,
	// change motors to use revolutions per second instead of rads/second
	VER_UE4_FIXUP_MOTOR_UNITS,
	// deprecated MovementComponent functions including "ModifiedMaxSpeed" et al
	VER_UE4_DEPRECATED_MOVEMENTCOMPONENT_MODIFIED_SPEEDS,
	// rename CanBeCharacterBase
	VER_UE4_RENAME_CANBECHARACTERBASE,
	// Change GameplayTagContainers to have FGameplayTags instead of FNames; Required to fix-up native serialization
	VER_UE4_GAMEPLAY_TAG_CONTAINER_TAG_TYPE_CHANGE,
	// Change from UInstancedFoliageSettings to UFoliageType, and change the api from being keyed on UStaticMesh* to UFoliageType*
	VER_UE4_FOLIAGE_SETTINGS_TYPE,
	// Lights serialize static shadow depth maps
	VER_UE4_STATIC_SHADOW_DEPTH_MAPS,
	// Add RF_Transactional to data assets, fixing undo problems when editing them
	VER_UE4_ADD_TRANSACTIONAL_TO_DATA_ASSETS,
	// Change LB_AlphaBlend to LB_WeightBlend in ELandscapeLayerBlendType
	VER_UE4_ADD_LB_WEIGHTBLEND,
	// Add root component to an foliage actor, all foliage cluster components will be attached to a root
	VER_UE4_ADD_ROOTCOMPONENT_TO_FOLIAGEACTOR,
	// FMaterialInstanceBasePropertyOverrides didn't use proper UObject serialize
	VER_UE4_FIX_MATERIAL_PROPERTY_OVERRIDE_SERIALIZE,
	// Addition of linear color sampler. color sample type is changed to linear sampler if source texture !sRGB
	VER_UE4_ADD_LINEAR_COLOR_SAMPLER,
	// Added StringAssetReferencesMap to support renames of FStringAssetReference properties.
	VER_UE4_ADD_STRING_ASSET_REFERENCES_MAP,
	// Apply scale from SCS RootComponent details in the Blueprint Editor to new actor instances at construction time
	VER_UE4_BLUEPRINT_USE_SCS_ROOTCOMPONENT_SCALE,
	// Changed level streaming to have a linear color since the visualization doesn't gamma correct.
	VER_UE4_LEVEL_STREAMING_DRAW_COLOR_TYPE_CHANGE,
	// Cleared end triggers from non-state anim notifies
	VER_UE4_CLEAR_NOTIFY_TRIGGERS,
	// Convert old curve names stored in anim assets into skeleton smartnames
	VER_UE4_SKELETON_ADD_SMARTNAMES,
	// Added the currency code field to FTextHistory_AsCurrency
	VER_UE4_ADDED_CURRENCY_CODE_TO_FTEXT,
	// Added support for C++11 enum classes
	VER_UE4_ENUM_CLASS_SUPPORT,
	// Fixup widget animation class
	VER_UE4_FIXUP_WIDGET_ANIMATION_CLASS,
	// USoundWave objects now contain details about compression scheme used.
	VER_UE4_SOUND_COMPRESSION_TYPE_ADDED,
	// Bodies will automatically weld when attached
	VER_UE4_AUTO_WELDING,
	// Rename UCharacterMovementComponent::bCrouchMovesCharacterDown
	VER_UE4_RENAME_CROUCHMOVESCHARACTERDOWN,
	// Lightmap parameters in FMeshBuildSettings
	VER_UE4_LIGHTMAP_MESH_BUILD_SETTINGS,
	// Rename SM3 to ES3_1 and updates featurelevel material node selector
	VER_UE4_RENAME_SM3_TO_ES3_1,
	// Deprecated separate style assets for use in UMG
	VER_UE4_DEPRECATE_UMG_STYLE_ASSETS,
	// Duplicating Blueprints will regenerate NodeGuids after this version
	VER_UE4_POST_DUPLICATE_NODE_GUID,
	// Rename USpringArmComponent::bUseControllerViewRotation to bUsePawnViewRotation,
	// Rename UCameraComponent::bUseControllerViewRotation to bUsePawnViewRotation (and change the default value)
	VER_UE4_RENAME_CAMERA_COMPONENT_VIEW_ROTATION,
	// Changed FName to be case preserving
	VER_UE4_CASE_PRESERVING_FNAME,
	// Rename USpringArmComponent::bUsePawnViewRotation to bUsePawnControlRotation
	// Rename UCameraComponent::bUsePawnViewRotation to bUsePawnControlRotation
	VER_UE4_RENAME_CAMERA_COMPONENT_CONTROL_ROTATION,
	// Fix bad refraction material attribute masks
	VER_UE4_FIX_REFRACTION_INPUT_MASKING,
	// A global spawn rate for emitters.
	VER_UE4_GLOBAL_EMITTER_SPAWN_RATE_SCALE,
	// Cleanup destructible mesh settings
	VER_UE4_CLEAN_DESTRUCTIBLE_SETTINGS,
	// CharacterMovementComponent refactor of AdjustUpperHemisphereImpact and deprecation of some associated vars.
	VER_UE4_CHARACTER_MOVEMENT_UPPER_IMPACT_BEHAVIOR,
	// Changed Blueprint math equality functions for vectors and rotators to operate as a "nearly" equals rather than "exact"
	VER_UE4_BP_MATH_VECTOR_EQUALITY_USES_EPSILON,
	// Static lighting support was re-added to foliage, and mobility was returned to static
	VER_UE4_FOLIAGE_STATIC_LIGHTING_SUPPORT,
	// Added composite fonts to Slate font info
	VER_UE4_SLATE_COMPOSITE_FONTS,
	// Remove UDEPRECATED_SaveGameSummary, required for UWorld::Serialize
	VER_UE4_REMOVE_SAVEGAMESUMMARY,

	//Remove bodyseutp serialization from skeletal mesh component
	VER_UE4_REMOVE_SKELETALMESH_COMPONENT_BODYSETUP_SERIALIZATION,
	// Made Slate font data use bulk data to store the embedded font data
	VER_UE4_SLATE_BULK_FONT_DATA,
	// Add new friction behavior in ProjectileMovementComponent.
	VER_UE4_ADD_PROJECTILE_FRICTION_BEHAVIOR,
	// Add axis settings enum to MovementComponent.
	VER_UE4_MOVEMENTCOMPONENT_AXIS_SETTINGS,
	// Switch to new interactive comments, requires boundry conversion to preserve previous states
	VER_UE4_GRAPH_INTERACTIVE_COMMENTBUBBLES,
	// Landscape serializes physical materials for collision objects 
	VER_UE4_LANDSCAPE_SERIALIZE_PHYSICS_MATERIALS,
	// Rename Visiblity on widgets to Visibility
	VER_UE4_RENAME_WIDGET_VISIBILITY,
	// add track curves for animation
	VER_UE4_ANIMATION_ADD_TRACKCURVES,
	// Removed BranchingPoints from AnimMontages and converted them to regular AnimNotifies.
	VER_UE4_MONTAGE_BRANCHING_POINT_REMOVAL,
	// Enforce const-correctness in Blueprint implementations of native C++ const class methods
	VER_UE4_BLUEPRINT_ENFORCE_CONST_IN_FUNCTION_OVERRIDES,
	// Added pivot to widget components, need to load old versions as a 0,0 pivot, new default is 0.5,0.5
	VER_UE4_ADD_PIVOT_TO_WIDGET_COMPONENT,
	// Added finer control over when AI Pawns are automatically possessed. Also renamed Pawn.AutoPossess to Pawn.AutoPossessPlayer indicate this was a setting for players and not AI.
	VER_UE4_PAWN_AUTO_POSSESS_AI,
	// Added serialization of timezone to FTextHistory for AsDate operations.
	VER_UE4_FTEXT_HISTORY_DATE_TIMEZONE,
	// Sort ActiveBoneIndices on lods so that we can avoid doing it at run time
	VER_UE4_SORT_ACTIVE_BONE_INDICES,
	// Added per-frame material uniform expressions
	VER_UE4_PERFRAME_MATERIAL_UNIFORM_EXPRESSIONS,
	// Make MikkTSpace the default tangent space calculation method for static meshes.
	VER_UE4_MIKKTSPACE_IS_DEFAULT,
	// Only applies to cooked files, grass cooking support.
	VER_UE4_LANDSCAPE_GRASS_COOKING,
	// Fixed code for using the bOrientMeshEmitters property.
	VER_UE4_FIX_SKEL_VERT_ORIENT_MESH_PARTICLES,
	// Do not change landscape section offset on load under world composition
	VER_UE4_LANDSCAPE_STATIC_SECTION_OFFSET,
	// New options for navigation data runtime generation (static, modifiers only, dynamic)
	VER_UE4_ADD_MODIFIERS_RUNTIME_GENERATION,
	// Tidied up material's handling of masked blend mode.
	VER_UE4_MATERIAL_MASKED_BLENDMODE_TIDY,
	// Original version of VER_UE4_MERGED_ADD_MODIFIERS_RUNTIME_GENERATION_TO_4_7; renumbered to prevent blocking promotion in main.
	VER_UE4_MERGED_ADD_MODIFIERS_RUNTIME_GENERATION_TO_4_7_DEPRECATED,
	// Original version of VER_UE4_AFTER_MERGED_ADD_MODIFIERS_RUNTIME_GENERATION_TO_4_7; renumbered to prevent blocking promotion in main.
	VER_UE4_AFTER_MERGED_ADD_MODIFIERS_RUNTIME_GENERATION_TO_4_7_DEPRECATED,
	// After merging VER_UE4_ADD_MODIFIERS_RUNTIME_GENERATION into 4.7 branch
	VER_UE4_MERGED_ADD_MODIFIERS_RUNTIME_GENERATION_TO_4_7,
	// After merging VER_UE4_ADD_MODIFIERS_RUNTIME_GENERATION into 4.7 branch
	VER_UE4_AFTER_MERGING_ADD_MODIFIERS_RUNTIME_GENERATION_TO_4_7,
	// Landscape grass weightmap data is now generated in the editor and serialized.
	VER_UE4_SERIALIZE_LANDSCAPE_GRASS_DATA,
	// New property to optionally prevent gpu emitters clearing existing particles on Init().
	VER_UE4_OPTIONALLY_CLEAR_GPU_EMITTERS_ON_INIT,
	// Also store the Material guid with the landscape grass data
	VER_UE4_SERIALIZE_LANDSCAPE_GRASS_DATA_MATERIAL_GUID,
	// Make sure that all template components from blueprint generated classes are flagged as public
	VER_UE4_BLUEPRINT_GENERATED_CLASS_COMPONENT_TEMPLATES_PUBLIC,
	// Split out creation method on ActorComponents to distinguish between native, instance, and simple or user construction script
	VER_UE4_ACTOR_COMPONENT_CREATION_METHOD,
	// K2Node_Event now uses FMemberReference for handling references
	VER_UE4_K2NODE_EVENT_MEMBER_REFERENCE,
	// FPropertyTag stores GUID of struct
	VER_UE4_STRUCT_GUID_IN_PROPERTY_TAG,
	// Remove unused UPolys from UModel cooked content
	VER_UE4_REMOVE_UNUSED_UPOLYS_FROM_UMODEL,
	// This doesn't do anything except trigger a rebuild on HISMC cluster trees, in this case to get a good "occlusion query" level
	VER_UE4_REBUILD_HIERARCHICAL_INSTANCE_TREES,
	// Package summary includes an CompatibleWithEngineVersion field, separately to the version it's saved with
	VER_UE4_PACKAGE_SUMMARY_HAS_COMPATIBLE_ENGINE_VERSION,
	// Track UCS modified properties on Actor Components
	VER_UE4_TRACK_UCS_MODIFIED_PROPERTIES,
	// Allowed landscape spline meshes to be stored into landscape streaming levels rather than the spline's level
	VER_UE4_LANDSCAPE_SPLINE_CROSS_LEVEL_MESHES,
	// Deprecate the variables used for sizing in the designer on UUserWidget
	VER_UE4_DEPRECATE_USER_WIDGET_DESIGN_SIZE,
	// Make the editor views array dynamically sized
	VER_UE4_ADD_EDITOR_VIEWS,
	// Updated foliage to work with either FoliageType assets or blueprint classes
	VER_UE4_FOLIAGE_WITH_ASSET_OR_CLASS,
	// Allows PhysicsSerializer to serialize shapes and actors for faster load times
	VER_UE4_BODYINSTANCE_BINARY_SERIALIZATION,
	// Added fastcall data serialization directly in UFunction
	VER_UE4_SERIALIZE_BLUEPRINT_EVENTGRAPH_FASTCALLS_IN_UFUNCTION,
	// Changes to USplineComponent and FInterpCurve
	VER_UE4_INTERPCURVE_SUPPORTS_LOOPING,
	// Material Instances overriding base material LOD transitions
	VER_UE4_MATERIAL_INSTANCE_BASE_PROPERTY_OVERRIDES_DITHERED_LOD_TRANSITION,
	// Serialize ES2 textures separately rather than overwriting the properties used on other platforms
	VER_UE4_SERIALIZE_LANDSCAPE_ES2_TEXTURES,
	// Constraint motor velocity is broken into per-component
	VER_UE4_CONSTRAINT_INSTANCE_MOTOR_FLAGS,
	// Serialize bIsConst in FEdGraphPinType
	VER_UE4_SERIALIZE_PINTYPE_CONST,
	// Change UMaterialFunction::LibraryCategories to LibraryCategoriesText (old assets were saved before auto-conversion of FArrayProperty was possible)
	VER_UE4_LIBRARY_CATEGORIES_AS_FTEXT,
	// Check for duplicate exports while saving packages.
	VER_UE4_SKIP_DUPLICATE_EXPORTS_ON_SAVE_PACKAGE,
	// Pre-gathering of gatherable, localizable text in packages to optimize text gathering operation times
	VER_UE4_SERIALIZE_TEXT_IN_PACKAGES,
	// Added pivot to widget components, need to load old versions as a 0,0 pivot, new default is 0.5,0.5
	VER_UE4_ADD_BLEND_MODE_TO_WIDGET_COMPONENT,
	// Added lightmass primitive setting
	VER_UE4_NEW_LIGHTMASS_PRIMITIVE_SETTING,
	// Deprecate NoZSpring property on spring nodes to be replaced with TranslateZ property
	VER_UE4_REPLACE_SPRING_NOZ_PROPERTY,
	// Keep enums tight and serialize their values as pairs of FName and value. Don't insert dummy values.
	VER_UE4_TIGHTLY_PACKED_ENUMS,
	// Changed Asset import data to serialize file meta data as JSON
	VER_UE4_ASSET_IMPORT_DATA_AS_JSON,
	// Legacy gamma support for textures.
	VER_UE4_TEXTURE_LEGACY_GAMMA,
	// Added WithSerializer for basic native structures like FVector, FColor etc to improve serialization performance
	VER_UE4_ADDED_NATIVE_SERIALIZATION_FOR_IMMUTABLE_STRUCTURES,
	// Deprecated attributes that override the style on UMG widgets
	VER_UE4_DEPRECATE_UMG_STYLE_OVERRIDES,
	// Shadowmap penumbra size stored
	VER_UE4_STATIC_SHADOWMAP_PENUMBRA_SIZE,
	// Fix BC on Niagara effects from the data object and dev UI changes.
	VER_UE4_NIAGARA_DATA_OBJECT_DEV_UI_FIX,
	// Fixed the default orientation of widget component so it faces down +x
	VER_UE4_FIXED_DEFAULT_ORIENTATION_OF_WIDGET_COMPONENT,
	// Removed bUsedWithUI flag from UMaterial and replaced it with a new material domain for UI
	VER_UE4_REMOVED_MATERIAL_USED_WITH_UI_FLAG,
	// Added braking friction separate from turning friction.
	VER_UE4_CHARACTER_MOVEMENT_ADD_BRAKING_FRICTION,
	// Removed TTransArrays from UModel
	VER_UE4_BSP_UNDO_FIX,
	// Added default value to dynamic parameter.
	VER_UE4_DYNAMIC_PARAMETER_DEFAULT_VALUE,
	// Added ExtendedBounds to StaticMesh
	VER_UE4_STATIC_MESH_EXTENDED_BOUNDS,
	// Added non-linear blending to anim transitions, deprecating old types
	VER_UE4_ADDED_NON_LINEAR_TRANSITION_BLENDS,
	// AO Material Mask texture
	VER_UE4_AO_MATERIAL_MASK,
	// Replaced navigation agents selection with single structure
	VER_UE4_NAVIGATION_AGENT_SELECTOR,
	// Mesh particle collisions consider particle size.
	VER_UE4_MESH_PARTICLE_COLLISIONS_CONSIDER_PARTICLE_SIZE,
	// Adjacency buffer building no longer automatically handled based on triangle count, user-controlled
	VER_UE4_BUILD_MESH_ADJ_BUFFER_FLAG_EXPOSED,
	// Change the default max angular velocity
	VER_UE4_MAX_ANGULAR_VELOCITY_DEFAULT,
	// Build Adjacency index buffer for clothing tessellation
	VER_UE4_APEX_CLOTH_TESSELLATION,
	// Added DecalSize member, solved backward compatibility
	VER_UE4_DECAL_SIZE,
	// Keep only package names in StringAssetReferencesMap
	VER_UE4_KEEP_ONLY_PACKAGE_NAMES_IN_STRING_ASSET_REFERENCES_MAP,
	// Support sound cue not saving out editor only data
	VER_UE4_COOKED_ASSETS_IN_EDITOR_SUPPORT,
	// Updated dialogue wave localization gathering logic.
	VER_UE4_DIALOGUE_WAVE_NAMESPACE_AND_CONTEXT_CHANGES,
	// Renamed MakeRot MakeRotator and rearranged parameters.
	VER_UE4_MAKE_ROT_RENAME_AND_REORDER,
	// K2Node_Variable will properly have the VariableReference Guid set if available
	VER_UE4_K2NODE_VAR_REFERENCEGUIDS,
	// Added support for sound concurrency settings structure and overrides
	VER_UE4_SOUND_CONCURRENCY_PACKAGE,
	// Changing the default value for focusable user widgets to false
	VER_UE4_USERWIDGET_DEFAULT_FOCUSABLE_FALSE,
	// Custom event nodes implicitly set 'const' on array and non-array pass-by-reference input params
	VER_UE4_BLUEPRINT_CUSTOM_EVENT_CONST_INPUT,
	// Renamed HighFrequencyGain to LowPassFilterFrequency
	VER_UE4_USE_LOW_PASS_FILTER_FREQ,
	// UAnimBlueprintGeneratedClass can be replaced by a dynamic class. Use TSubclassOf<UAnimInstance> instead.
	VER_UE4_NO_ANIM_BP_CLASS_IN_GAMEPLAY_CODE,
	// The SCS keeps a list of all nodes in its hierarchy rather than recursively building it each time it is requested
	VER_UE4_SCS_STORES_ALLNODES_ARRAY,
	// Moved StartRange and EndRange in UFbxAnimSequenceImportData to use FInt32Interval
	VER_UE4_FBX_IMPORT_DATA_RANGE_ENCAPSULATION,
	// Adding a new root scene component to camera component
	VER_UE4_CAMERA_COMPONENT_ATTACH_TO_ROOT,
	// Updating custom material expression nodes for instanced stereo implementation
	VER_UE4_INSTANCED_STEREO_UNIFORM_UPDATE,
	// Texture streaming min and max distance to handle HLOD
	VER_UE4_STREAMABLE_TEXTURE_MIN_MAX_DISTANCE,	
	// Fixing up invalid struct-to-struct pin connections by injecting available conversion nodes
	VER_UE4_INJECT_BLUEPRINT_STRUCT_PIN_CONVERSION_NODES,
	// Saving tag data for Array Property's inner property
	VER_UE4_INNER_ARRAY_TAG_INFO,
	// Fixed duplicating slot node names in skeleton due to skeleton preload on compile
	VER_UE4_FIX_SLOT_NAME_DUPLICATION,
	// Texture streaming using AABBs instead of Spheres
	VER_UE4_STREAMABLE_TEXTURE_AABB,
	// FPropertyTag stores GUID of property
	VER_UE4_PROPERTY_GUID_IN_PROPERTY_TAG,
	// Name table hashes are calculated and saved out rather than at load time
	VER_UE4_NAME_HASHES_SERIALIZED,
	// Updating custom material expression nodes for instanced stereo implementation refactor
	VER_UE4_INSTANCED_STEREO_UNIFORM_REFACTOR,
	// Added compression to the shader resource for memory savings
	VER_UE4_COMPRESSED_SHADER_RESOURCES,
	// Cooked files contain the dependency graph for the event driven loader (the serialization is largely independent of the use of the new loader)
	VER_UE4_PRELOAD_DEPENDENCIES_IN_COOKED_EXPORTS,
	// Cooked files contain the TemplateIndex used by the event driven loader (the serialization is largely independent of the use of the new loader, i.e. this will be null if cooking for the old loader)
	VER_UE4_TemplateIndex_IN_COOKED_EXPORTS,
	// FPropertyTag includes contained type(s) for Set and Map properties:
	VER_UE4_PROPERTY_TAG_SET_MAP_SUPPORT,
	// Added SearchableNames to the package summary and asset registry
	VER_UE4_ADDED_SEARCHABLE_NAMES,
	// Increased size of SerialSize and SerialOffset in export map entries to 64 bit, allow support for bigger files
	VER_UE4_64BIT_EXPORTMAP_SERIALSIZES,
	// Sky light stores IrradianceMap for mobile renderer.
	VER_UE4_SKYLIGHT_MOBILE_IRRADIANCE_MAP,
	// Added flag to control sweep behavior while walking in UCharacterMovementComponent.
	VER_UE4_ADDED_SWEEP_WHILE_WALKING_FLAG,
	// StringAssetReference changed to SoftObjectPath and swapped to serialize as a name+string instead of a string
	VER_UE4_ADDED_SOFT_OBJECT_PATH,
	// Changed the source orientation of point lights to match spot lights (z axis)
	VER_UE4_POINTLIGHT_SOURCE_ORIENTATION,
	// LocalizationId has been added to the package summary (editor-only)
	VER_UE4_ADDED_PACKAGE_SUMMARY_LOCALIZATION_ID,
	// Fixed case insensitive hashes of wide strings containing character values from 128-255
	VER_UE4_FIX_WIDE_STRING_CRC,
	// Added package owner to allow private references
	VER_UE4_ADDED_PACKAGE_OWNER,
	// Changed the data layout for skin weight profile data
	VER_UE4_SKINWEIGHT_PROFILE_DATA_LAYOUT_CHANGES,
	// Added import that can have package different than their outer
	VER_UE4_NON_OUTER_PACKAGE_IMPORT,
	// Added DependencyFlags to AssetRegistry
	VER_UE4_ASSETREGISTRY_DEPENDENCYFLAGS,
	// Fixed corrupt licensee flag in 4.26 assets
	VER_UE4_CORRECT_LICENSEE_FLAG,

	// -----<new versions can be added before this line>-------------------------------------------------
	// - this needs to be the last line (see note below)
	VER_UE4_AUTOMATIC_VERSION_PLUS_ONE,
	VER_UE4_AUTOMATIC_VERSION = VER_UE4_AUTOMATIC_VERSION_PLUS_ONE - 1
};

enum EUnrealEngineObjectLicenseeUEVersion
{
	VER_LIC_NONE = 0,
	// - this needs to be the last line (see note below)
	VER_LIC_AUTOMATIC_VERSION_PLUS_ONE,
	VER_LIC_AUTOMATIC_VERSION = VER_LIC_AUTOMATIC_VERSION_PLUS_ONE - 1
};

static_assert(EUnrealEngineObjectUE4Version::VER_UE4_AUTOMATIC_VERSION < (int32)EUnrealEngineObjectUE5Version::INITIAL_VERSION, "UE4 versioning must be lower than the start of UE5 versioning");

/**
 * This object combines all of our version enums into a single easy to use structure
 * which allows us to update older version numbers independently of the newer version numbers.
 */
struct FPackageFileVersion
{
	FPackageFileVersion() = default;
	FPackageFileVersion(int32 UE4Version, EUnrealEngineObjectUE5Version UE5Version)
		: FileVersionUE4(UE4Version)
		, FileVersionUE5((int32)UE5Version)
	{

	}

	/** Set all versions to the default state */
	void Reset()
	{
		FileVersionUE4 = 0;
		FileVersionUE5 = 0;
	}

	/** Creates and returns a FPackageFileVersion based on a single EUnrealEngineObjectUEVersion and no other versions. */
	CORE_API static FPackageFileVersion CreateUE4Version(int32 Version);
	CORE_API static FPackageFileVersion CreateUE4Version(EUnrealEngineObjectUE4Version Version);

	/** Returns the highest valid version number which is considered to be the 'true' version number */
	int32 ToValue() const
	{
		if (FileVersionUE5 >= (int32)EUnrealEngineObjectUE5Version::INITIAL_VERSION)
		{
			return FileVersionUE5;
		}
		else
		{
			return FileVersionUE4;
		}
	}

	/** UE4 version comparisons */
	bool operator !=(EUnrealEngineObjectUE4Version Version) const
	{
		return FileVersionUE4 != Version;
	}

	/** UE4 version comparisons */
	bool operator <(EUnrealEngineObjectUE4Version Version) const
	{
		return FileVersionUE4 < Version;
	}

	/** UE4 version comparisons */
	bool operator >=(EUnrealEngineObjectUE4Version Version) const
	{
		return FileVersionUE4 >= Version;
	}

	/** UE5 version comparisons */
	bool operator !=(EUnrealEngineObjectUE5Version Version) const
	{
		return FileVersionUE5 != (int32)Version;
	}

	/** UE5 version comparisons  */
	bool operator <(EUnrealEngineObjectUE5Version Version) const
	{
		return FileVersionUE5 < (int32)Version;
	}

	/** UE5 version comparisons */
	bool operator >=(EUnrealEngineObjectUE5Version Version) const
	{
		return FileVersionUE5 >= (int32)Version;
	}

	/**
	 * Returns true if this object is compatible with the FPackageFileVersion passed in as the parameter.
	 * This means that  all version numbers for the current object are equal or greater than the
	 * corresponding version numbers of the other structure.
	 */
	bool IsCompatible(const FPackageFileVersion& Other) const
	{
		return FileVersionUE4 >= Other.FileVersionUE4 && FileVersionUE5 >= Other.FileVersionUE5;
	}

	/** FPackageFileVersion comparisons */
	bool operator==(const FPackageFileVersion& Other) const
	{
		return FileVersionUE4 == Other.FileVersionUE4 && FileVersionUE5 == Other.FileVersionUE5;
	}

	/** FPackageFileVersion comparisons */
	bool operator!=(const FPackageFileVersion& Other) const
	{
		return !(*this == Other);
	}

	/** Serialization members */
	CORE_API friend FArchive& operator<<(FArchive& Ar, FPackageFileVersion& Version);
	/** Serialization members implemented as hidden friends*/
private:
	friend FCbWriter& operator<<(FCbWriter& Writer, const FPackageFileVersion& Version)
	{
		return Version.Write(Writer);
	}
	CORE_API FCbWriter& Write(FCbWriter& Writer) const;
	friend bool LoadFromCompactBinary(const FCbFieldView& FieldView, FPackageFileVersion& Version)
	{
		return Version.TryRead(FieldView);
	}
	CORE_API bool TryRead(const FCbFieldView& FieldView);
public:

	/* UE4 file version*/
	int32		FileVersionUE4 = 0;
	/* UE5 file version */
	int32		FileVersionUE5 = 0;
};

#define VER_LATEST_ENGINE_UE4           PREPROCESSOR_ENUM_PROTECT(VER_UE4_AUTOMATIC_VERSION)
#define VER_LATEST_ENGINE_UE5           PREPROCESSOR_ENUM_PROTECT(EUnrealEngineObjectUE5Version::AUTOMATIC_VERSION)
#define VER_UE4_DEPRECATED_PACKAGE      PREPROCESSOR_ENUM_PROTECT(VER_UE4_OLDEST_LOADABLE_PACKAGE)
#define VER_LATEST_ENGINE_LICENSEEUE4   PREPROCESSOR_ENUM_PROTECT(VER_LIC_AUTOMATIC_VERSION)

// Minimum package version that contains legal bytecode
#define VER_MIN_SCRIPTVM_UE4									(VER_UE4_CASE_PRESERVING_FNAME)
#define VER_MIN_SCRIPTVM_LICENSEEUE4							(VER_LIC_NONE)

// Version access.
extern const CORE_API FPackageFileVersion	GPackageFileUEVersion;				// Unreal Version Number.
extern const CORE_API FPackageFileVersion	GOldestLoadablePackageFileUEVersion;// The oldest Unreal Version number that we can load.
extern const CORE_API int32					GPackageFileLicenseeUEVersion;		// Licensee Version Number.
