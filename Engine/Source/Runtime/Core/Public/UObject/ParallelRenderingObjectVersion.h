// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DevObjectVersion.h"
#include "Containers/Map.h"

// System Guids for changes made in the Dev-ParallelRendering stream
struct FParallelRenderingObjectVersion
{
	static CORE_API TMap<FGuid, FGuid> GetSystemGuids();

private:
	FParallelRenderingObjectVersion() {}
};
