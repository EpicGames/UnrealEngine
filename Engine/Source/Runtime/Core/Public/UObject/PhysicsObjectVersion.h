// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes made in Release-4.26-Chaos 
// Previously used for Dev-Physics stream
struct FPhysicsObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,
		// Adding PerShapeData to serialization
		PerShapeData,
		// Add serialization from handle back to particle
		SerializeGTGeometryParticles,
		// Groom serialization with hair description as bulk data
		GroomWithDescription,
		// Groom serialization with import option
		GroomWithImportSettings,

		// TriangleMesh has map from source vertex index to internal vertex index for per-poly collisoin.
		TriangleMeshHasVertexIndexMap,

		// Chaos Convex StructureData supports different index sizes based on num verts/planes
		VariableConvexStructureData,

		// Add the ability to enable or disable Continuous Collision Detection
		AddCCDEnableFlag,

		// Added the weighted value property type to store the cloths weight maps' low/high ranges
		ChaosClothAddWeightedValue,

		// Chaos FConvex uses array of FVec3s for vertices instead of particles
		ConvexUsesVerticesArray,

		// Add centrifugal forces for cloth
		ChaosClothAddfictitiousforces,

		// Added the Long Range Attachment stiffness weight map
		ChaosClothAddTetherStiffnessWeightMap,

		// Fix corrupted LOD transition maps
		ChaosClothFixLODTransitionMaps,

		// Convex structure data is now an index-based half-edge structure
		ChaosConvexUsesHalfEdges,

		// Convex structure data has a list of unique edges (half of the half edges)
		ChaosConvexHasUniqueEdgeSet,

		// Chaos FGeometryCollectionObject user defined collision shapes support
		GeometryCollectionUserDefinedCollisionShapes,

		// Chaos Remove scale from TKinematicTarget object
		ChaosKinematicTargetRemoveScale,

		// Chaos Added support for per-object collision constraint flag.
		AddCollisionConstraintFlag,

		// Expose particle Disabled flag to the game thread
		AddDisabledFlag,

		// Added max linear and angular speed to Chaos bodies
		AddChaosMaxLinearAngularSpeed,

		// add convex geometry to older collections that did not have any
		GeometryCollectionConvexDefaults,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

private:
	FPhysicsObjectVersion() {}
};
