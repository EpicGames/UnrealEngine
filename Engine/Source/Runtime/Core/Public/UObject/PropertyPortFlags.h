// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PropertyPortFlags.h: Property import/export flags.
=============================================================================*/

#pragma once

#include "CoreTypes.h"

// Property exporting flags.
enum EPropertyPortFlags
{
	/** No special property exporting flags */
	PPF_None						= 0x00000000,

	/** Indicates that property data should be wrapped in quotes (for some types of properties) */
	PPF_Delimited					= 0x00000001,

	/** Indicates that the object reference should be verified */
	PPF_CheckReferences				= 0x00000002, 
	
	PPF_ExportsNotFullyQualified	= 0x00000004,
	
	PPF_AttemptNonQualifiedSearch	= 0x00000008,
	
	/** Indicates that importing values for config properties is disallowed */
	PPF_RestrictImportTypes			= 0x00000010,

	/** Indicates that this is a blueprint pin or something else that is saved to disk as import text */
	PPF_SerializedAsImportText		= 0x00000020,

	/** 
	 * Set if this is being exported/imported for an external editor format like CSV.
	 * If true, it uses authored names instead of internal names and default values are always written out.
	 */
	PPF_ExternalEditor				= 0x00000040,

	/** only include properties which are marked CPF_InstancedReference */
	PPF_SubobjectsOnly				= 0x00000080,

	/**
	 * Indicates that two object should be considered identical
	 * if the property values for both objects are all identical
	 */
	PPF_DeepComparison				= 0x00000100,

	/**
	 * Similar to PPF_DeepComparison, except that template components are always compared using standard object
	 * property comparison logic (basically if the pointers are different, then the property isn't identical)
	 */
	PPF_DeepCompareInstances		= 0x00000200,

	/**
	 * Only do a deep comparison if the objects are native created default subobjects.
	 * This can be used to use the legacy FPropertyObjectBase::Identical behavior where subobjects that were
	 * not created as a default subobject would always return false for a deep compare.
	 * Requires PPF_DeepComparison or PPF_DeepCompareInstances to also be set
	 */
	 PPF_DeepCompareDSOsOnly		= 0x00000400,

	/**
	 * Set if this operation is copying in memory (for copy/paste) instead of exporting to a file. There are
	 * some subtle differences between the two
	 */
	PPF_Copy						= 0x00000800,

	/** Set when duplicating objects via serialization */
	PPF_Duplicate					= 0x00001000,

	/** Indicates that object property values should be exported without the package or class information */
	PPF_SimpleObjectText			= 0x00002000,

	/** Set if this export is going to be used for diffing and may want to omit certain data that doesn't make sense for comparison */
	PPF_ForDiff						= 0x00004000,
	
	/**
	 * Set if this export is going to be used for diffing and may want to omit certain data related to CDOs/archetypes that doesn't make sense when comparing an instance of an object/property/value.
	 * Requires PPF_ForDiff to also be set.
	 */
	PPF_ForDiffInstanceOnly			= 0x00008000,

	/** parsing default properties - allow text for transient properties to be imported - also modifies ObjectProperty importing slightly for subobjects */
	PPF_ParsingDefaultProperties	= 0x00010000,
	
	/** indicates that non-categorized transient properties should be exported (by default, they would not be) */
	PPF_IncludeTransient			= 0x00020000,

	/** Indicates that any instanced subobject properties should get a duplicate of the specified object rather than having its value set to the specified text */
	PPF_InstanceSubobjects			= 0x00040000,

	/** indicates that we're exporting properties for display in the property window. - used to hide EditHide items in collapsed structs */
	PPF_PropertyWindow				= 0x00080000,

	/** Avoid dereferencing remote object references if possible */
	PPF_AvoidRemoteObjectMigration	= 0x00100000,

	/** Force fully qualified object names (for debug dumping) */
	PPF_DebugDump					= 0x00200000,

	/** Set when duplicating objects for PIE */
	PPF_DuplicateForPIE				= 0x00400000,

	/** Set when exporting just an object declaration, to be followed by another call with PPF_SeparateDefine */
	PPF_SeparateDeclare				= 0x00800000,

	/** Set when exporting just an object definition, preceded by another call with PPF_SeparateDeclare */
	PPF_SeparateDefine				= 0x01000000,

	/** Used by 'watch value' while blueprint debugging*/
	PPF_BlueprintDebugView			= 0x02000000,

	/** Exporting properties for console variables. */
	PPF_ConsoleVariable				= 0x04000000,

	/** Ignores CPF_Deprecated flag */
	PPF_UseDeprecatedProperties		= 0x08000000,

	//								= 0x10000000,

	/** Ignores CPF_SkipSerialization flag when using tagged serialization */
	PPF_ForceTaggedSerialization	= 0x20000000,

	/** Set when duplicating objects verbatim (doesn't reset unique IDs) */
	PPF_DuplicateVerbatim			= 0x40000000,

	//								= 0x80000000,
};
