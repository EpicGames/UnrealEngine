// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "UObject/UObjectHierarchyFwd.h"

// Maps an FProperty class (forward-declared in UObjectHierarchyFwd.h) to its registered FName.
// Uses pre-registered EName constants from UnrealNames.inl.
template <typename PropertyType>
struct TPropertyTypeToFName;

template <> struct TPropertyTypeToFName<FByteProperty>                    { static FName Get() { return NAME_ByteProperty; } };
template <> struct TPropertyTypeToFName<FInt8Property>                    { static FName Get() { return NAME_Int8Property; } };
template <> struct TPropertyTypeToFName<FInt16Property>                   { static FName Get() { return NAME_Int16Property; } };
template <> struct TPropertyTypeToFName<FIntProperty>                     { static FName Get() { return NAME_IntProperty; } };
template <> struct TPropertyTypeToFName<FInt64Property>                   { static FName Get() { return NAME_Int64Property; } };
template <> struct TPropertyTypeToFName<FUInt16Property>                  { static FName Get() { return NAME_UInt16Property; } };
template <> struct TPropertyTypeToFName<FUInt32Property>                  { static FName Get() { return NAME_UInt32Property; } };
template <> struct TPropertyTypeToFName<FUInt64Property>                  { static FName Get() { return NAME_UInt64Property; } };
template <> struct TPropertyTypeToFName<FBoolProperty>                    { static FName Get() { return NAME_BoolProperty; } };
template <> struct TPropertyTypeToFName<FFloatProperty>                   { static FName Get() { return NAME_FloatProperty; } };
template <> struct TPropertyTypeToFName<FDoubleProperty>                  { static FName Get() { return NAME_DoubleProperty; } };
template <> struct TPropertyTypeToFName<FObjectProperty>                  { static FName Get() { return NAME_ObjectProperty; } };
template <> struct TPropertyTypeToFName<FClassProperty>                   { static FName Get() { return NAME_ObjectProperty; } }; // ClassProperty shares ObjectProperty's tag
template <> struct TPropertyTypeToFName<FInterfaceProperty>               { static FName Get() { return NAME_InterfaceProperty; } };
template <> struct TPropertyTypeToFName<FWeakObjectProperty>              { static FName Get() { return NAME_WeakObjectProperty; } };
template <> struct TPropertyTypeToFName<FLazyObjectProperty>              { static FName Get() { return NAME_LazyObjectProperty; } };
template <> struct TPropertyTypeToFName<FSoftObjectProperty>              { static FName Get() { return NAME_SoftObjectProperty; } };
template <> struct TPropertyTypeToFName<FSoftClassProperty>               { static FName Get() { return NAME_SoftObjectProperty; } }; // SoftClassProperty shares SoftObjectProperty's tag
template <> struct TPropertyTypeToFName<FNameProperty>                    { static FName Get() { return NAME_NameProperty; } };
template <> struct TPropertyTypeToFName<FStructProperty>                  { static FName Get() { return NAME_StructProperty; } };
template <> struct TPropertyTypeToFName<FStrProperty>                     { static FName Get() { return NAME_StrProperty; } };
template <> struct TPropertyTypeToFName<FTextProperty>                    { static FName Get() { return NAME_TextProperty; } };
template <> struct TPropertyTypeToFName<FArrayProperty>                   { static FName Get() { return NAME_ArrayProperty; } };
template <> struct TPropertyTypeToFName<FDelegateProperty>                { static FName Get() { return NAME_DelegateProperty; } };
template <> struct TPropertyTypeToFName<FMulticastInlineDelegateProperty> { static FName Get() { return NAME_MulticastInlineDelegateProperty; } };
template <> struct TPropertyTypeToFName<FMulticastSparseDelegateProperty> { static FName Get() { return NAME_MulticastSparseDelegateProperty; } };
template <> struct TPropertyTypeToFName<FMapProperty>                     { static FName Get() { return NAME_MapProperty; } };
template <> struct TPropertyTypeToFName<FSetProperty>                     { static FName Get() { return NAME_SetProperty; } };
template <> struct TPropertyTypeToFName<FEnumProperty>                    { static FName Get() { return NAME_EnumProperty; } };
template <> struct TPropertyTypeToFName<FUtf8StrProperty>                 { static FName Get() { return NAME_Utf8StrProperty; } };
template <> struct TPropertyTypeToFName<FAnsiStrProperty>                 { static FName Get() { return NAME_AnsiStrProperty; } };
template <> struct TPropertyTypeToFName<FOptionalProperty>                { static FName Get() { return NAME_OptionalProperty; } };
template <> struct TPropertyTypeToFName<FFieldPathProperty>               { static FName Get() { return NAME_FieldPathProperty; } };
