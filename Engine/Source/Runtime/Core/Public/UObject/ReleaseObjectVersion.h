// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes made in Release streams.
// NOTE: Beware of patch releases - do not bump this except in the latest Release stream!
struct FReleaseObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// Static Mesh extended bounds radius fix
		StaticMeshExtendedBoundsFix,

		//Physics asset bodies are either in the sync scene or the async scene, but not both
		NoSyncAsyncPhysAsset,

		// ULevel was using TTransArray incorrectly (serializing the entire array in addition to individual mutations).
		// converted to a TArray:
		LevelTransArrayConvertedToTArray,

		// Add Component node templates now use their own unique naming scheme to ensure more reliable archetype lookups.
		AddComponentNodeTemplateUniqueNames,

		// Fix a serialization issue with static mesh FMeshSectionInfoMap FProperty
		UPropertryForMeshSectionSerialize,

		// Existing HLOD settings screen size to screen area conversion
		ConvertHLODScreenSize,

		// Adding mesh section info data for existing billboard LOD models
		SpeedTreeBillboardSectionInfoFixup,

		// Change FMovieSceneEventParameters::StructType to be a string asset reference from a TWeakObjectPtr<UScriptStruct>
		EventSectionParameterStringAssetRef,

		// Remove serialized irradiance map data from skylight.
		SkyLightRemoveMobileIrradianceMap,

		// rename bNoTwist to bAllowTwist
		RenameNoTwistToAllowTwistInTwoBoneIK,

		// Material layers serialization refactor
		MaterialLayersParameterSerializationRefactor,

		// Added disable flag to skeletal mesh data
		AddSkeletalMeshSectionDisable,

		// Removed objects that were serialized as part of this material feature
		RemovedMaterialSharedInputCollection,

		// HISMC Cluster Tree migration to add new data
		HISMCClusterTreeMigration,

		// Default values on pins in blueprints could be saved incoherently
		PinDefaultValuesVerified,

		// During copy and paste transition getters could end up with broken state machine references
		FixBrokenStateMachineReferencesInTransitionGetters,

		// Change to MeshDescription serialization
		MeshDescriptionNewSerialization,

		// Change to not clamp RGB values > 1 on linear color curves
		UnclampRGBColorCurves,

		// Bugfix for FAnimObjectVersion::LinkTimeAnimBlueprintRootDiscovery.
		LinkTimeAnimBlueprintRootDiscoveryBugFix,

		// Change trail anim node variable deprecation
		TrailNodeBlendVariableNameChange,

		// Make sure the Blueprint Replicated Property Conditions are actually serialized properly.
		PropertiesSerializeRepCondition,

		// DepthOfFieldFocalDistance at 0 now disables DOF instead of DepthOfFieldFstop at 0.
		FocalDistanceDisablesDOF,

		// Removed versioning, but version entry must still exist to keep assets saved with this version loadable
		Unused_SoundClass2DReverbSend,

		// Groom asset version
		GroomAssetVersion1,
		GroomAssetVersion2,

		// Store applied version of Animation Modifier to use when reverting
		SerializeAnimModifierState,

		// Groom asset version
		GroomAssetVersion3,

		// Upgrade filmback
		DeprecateFilmbackSettings,
		
		// custom collision type
		CustomImplicitCollisionType,

		// FFieldPath will serialize the owner struct reference and only a short path to its property
		FFieldPathOwnerSerialization,

		// New MeshDescription format
		// This was inadvertently added in UE5. The proper version for it is in in UE5MainStreamObjectVersion
		MeshDescriptionNewFormat,

		// Pin types include a flag that propagates the 'CPF_UObjectWrapper' flag to generated properties
		PinTypeIncludesUObjectWrapperFlag,

		// Added Weight member to FMeshToMeshVertData
		WeightFMeshToMeshVertData,

		// Animation graph node bindings displayed as pins
		AnimationGraphNodeBindingsDisplayedAsPins,

		// Serialized rigvm offset segment paths
		SerializeRigVMOffsetSegmentPaths,

		// Upgrade AbcGeomCacheImportSettings for velocities
		AbcVelocitiesSupport,

		// Add margin support to Chaos Convex
		MarginAddedToConvexAndBox,

		// Add structure data to Chaos Convex
		StructureDataAddedToConvex,

		// Changed axis UI for LiveLink AxisSwitch Pre Processor
		AddedFrontRightUpAxesToLiveLinkPreProcessor,

		// Some sequencer event sections that were copy-pasted left broken links to the director BP
		FixupCopiedEventSections,

		// Serialize the number of bytes written when serializing function arguments
		RemoteControlSerializeFunctionArgumentsSize,

		// Add loop counters to sequencer's compiled sub-sequence data
		AddedSubSequenceEntryWarpCounter,

		// Remove default resolution limit of 512 pixels for cubemaps generated from long-lat sources
		LonglatTextureCubeDefaultMaxResolution,

		// bake center of mass into chaos cache
		GeometryCollectionCacheRemovesMassToLocal,
			
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

private:
	FReleaseObjectVersion() {}
};
