// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

enum class EResidence : uint8
{
	Local /*UE_EXPERIMENTAL(5.8, "This enum type and its value names may change in the future")*/ = 0,			// The object is local and fully resolved (PostMigrated)
	LocalNotReady /*UE_EXPERIMENTAL(5.8, "This enum type and its value names may change in the future")*/ = 1,	// The object's memory is local but requires resolving (deserialization and PostMigrate)
	Remote /*UE_EXPERIMENTAL(5.8, "This enum type and its value names may change in the future")*/ = 2,			// The object is remote
};

CORE_API const TCHAR* EnumToString(EResidence InResidence);