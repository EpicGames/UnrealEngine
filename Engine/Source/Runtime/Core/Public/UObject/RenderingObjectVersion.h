// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes made in Dev-Rendering stream
struct FRenderingObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// Added support for 3 band SH in the ILC
		IndirectLightingCache3BandSupport,

		// Allows specifying resolution for reflection capture probes
		CustomReflectionCaptureResolutionSupport,

		RemovedTextureStreamingLevelData,

		// translucency is now a property which matters for materials with the decal domain
		IntroducedMeshDecals,

		// Reflection captures are no longer prenormalized
		ReflectionCapturesStoreAverageBrightness,

		ChangedPlanarReflectionFadeDefaults,

		RemovedRenderTargetSize,

		// Particle Cutout (SubUVAnimation) data is now stored in the ParticleRequired Module
		MovedParticleCutoutsToRequiredModule,

		MapBuildDataSeparatePackage,

		// StaticMesh and SkeletalMesh texcoord size data.
		TextureStreamingMeshUVChannelData,
		
		// Added type handling to material normalize and length (sqrt) nodes
		TypeHandlingForMaterialSqrtNodes,

		FixedBSPLightmaps,

		DistanceFieldSelfShadowBias,

		FixedLegacyMaterialAttributeNodeTypes,

		ShaderResourceCodeSharing,

		MotionBlurAndTAASupportInSceneCapture2d,

		AddedTextureRenderTargetFormats,

		// Triggers a rebuild of the mesh UV density while also adding an update in the postedit
		FixedMeshUVDensity,

		AddedbUseShowOnlyList,

		VolumetricLightmaps,

		MaterialAttributeLayerParameters,

		StoreReflectionCaptureBrightnessForCooking,

		// FModelVertexBuffer does serialize a regular TArray instead of a TResourceArray
		ModelVertexBufferSerialization,

		ReplaceLightAsIfStatic,

		// Added per FShaderType permutation id.
		ShaderPermutationId,

		// Changed normal precision in imported data
		IncreaseNormalPrecision,

		VirtualTexturedLightmaps,

		GeometryCacheFastDecoder,

		LightmapHasShadowmapData,

		// Removed old gaussian and bokeh DOF methods from deferred shading renderer.
		DiaphragmDOFOnlyForDeferredShadingRenderer,

		// Lightmaps replace ULightMapVirtualTexture (non-UTexture derived class) with ULightMapVirtualTexture2D (derived from UTexture)
		VirtualTexturedLightmapsV2,

		SkyAtmosphereStaticLightingVersioning,

		// UTextureRenderTarget2D now explicitly allows users to create sRGB or non-sRGB type targets
		ExplicitSRGBSetting,

		VolumetricLightmapStreaming,

		//ShaderModel4 support removed from engine
		RemovedSM4,

		// Deterministic ShaderMapID serialization
		MaterialShaderMapIdSerialization,

		// Add force opaque flag for static mesh
		StaticMeshSectionForceOpaqueField,

		// Add force opaque flag for static mesh
		AutoExposureChanges,

		// Removed emulated instancing from instanced static meshes
		RemovedEmulatedInstancing,

		// Added per instance custom data (for Instanced Static Meshes)
		PerInstanceCustomData,

		// Added material attributes to shader graph to support anisotropic materials 
		AnisotropicMaterial,

		// Add if anything has changed in the exposure, override the bias to avoid the new default propagating
		AutoExposureForceOverrideBiasFlag,

		// Override for a special case for objects that were serialized and deserialized between versions AutoExposureChanges and AutoExposureForceOverrideBiasFlag
		AutoExposureDefaultFix,

		// Remap Volume Extinction material input to RGB
		VolumeExtinctionBecomesRGB,

		// Add a new virtual texture to support virtual texture light map on mobile
		VirtualTexturedLightmapsV3,

		// Added fallback target modes including auto to Nanite builder
		NaniteFallbackTarget,

		// Added support to force Nanite material usage if Nanite mesh builders are also forced.
		NaniteForceMaterialUsage,

		// Custom material expression includes are now serialized & used for DDC key generation
		AddedMaterialExpressionIncludesHash,

		// Do not serialize deprecated high res source mesh
		DeprecatedHighResSourceMesh,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

private:
	FRenderingObjectVersion() {}
};
