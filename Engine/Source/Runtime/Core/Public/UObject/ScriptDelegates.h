// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/CompleteType.h"
#include "Concepts/Const.h"
#include "Concepts/Reference.h"
#include "Concepts/Scalar.h"
#include "Concepts/UnscopedEnum.h" // this can be removed when UE_DYNAMIC_DELEGATE_UNSCOPED_ENUM_HACK is removed
#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/EnumAsByte.h" // this can be removed when UE_DYNAMIC_DELEGATE_UNSCOPED_ENUM_HACK is removed
#include "Containers/UnrealString.h"
#include "PropertyPortFlags.h"
#include "Delegates/DelegateAccessHandler.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/NameTypes.h"
#include <type_traits>

/* Specifies whether or not dynamic delegates can have payloads. */
#ifndef UE_USE_DYNAMIC_DELEGATE_PAYLOADS
	#define UE_USE_DYNAMIC_DELEGATE_PAYLOADS 0

	/* Dynamic delegates with payloads and return values need special handling. */
	#ifndef UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		#define UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL 0
	#endif
#endif

/* A hack to force unscoped enums to be a TEnumAsByte.  This should be enforced by UHT but it wasn't being caught
 * for function parameters - including dynamic delegate declarations - so we force it here, otherwise the ProcessEvent
 * struct can end up misaligned.
 *
 * This can be removed when UHT rejects unscoped enum parameters in dynamic delegate declarations.
 */
#ifndef UE_DYNAMIC_DELEGATE_UNSCOPED_ENUM_HACK
	#define UE_DYNAMIC_DELEGATE_UNSCOPED_ENUM_HACK 1
#endif

#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
	#include "Concepts/Arithmetic.h"
	#include "Concepts/UEnum.h"
	#include "Concepts/UnscopedEnum.h"
	#include "Concepts/UObject.h"
	#include "Concepts/UScriptStruct.h"
	#include "Templates/Tuple.h"
	#include "Traits/FunctionArity.h"
	#include "UObject/DelegatePayloadSchema.h"
	#include "UObject/NativeTypeToPropertyType.h"
	#include "UObject/UObjectHierarchyFwd.h"
	#include "UObject/WeakObjectPtrTemplatesFwd.h"
#endif

// A suffix appended to the serialized function name to declare that a payload was written
// after it.  This is used instead of an explicit flag to avoid impacting the existing binary format.
#define UE_DYNAMIC_DELEGATE_PAYLOAD_FUNCTION_NAME_SUFFIX TEXT("[bHasPayload]")

namespace UE::Core::Private
{
	template <typename InThreadSafetyMode>
	struct TScriptDelegateTraits
	{
		// Although templated, WeakPtrType is not intended to be anything other than FWeakObjectPtr,
		// and is only a template for module organization reasons.
		using WeakPtrType = FWeakObjectPtr;

		using ThreadSafetyMode = InThreadSafetyMode;
		using UnicastThreadSafetyModeForMulticasts = FNotThreadSafeNotCheckedDelegateMode;
	};

#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
	// Can't use TWeakObjectPtr directly here, so take a guess it being one by checking for one of its unique members
	template <typename T>
	concept CWeakPointerish =
		requires(const T & Ptr)
	{
		Ptr.HasSameIndexAndSerialNumber(Ptr);
	};

	// Wrapper for TWeakObjectPtr that implicitly converts to and from its raw pointer, for use in payload captures.
	template <typename T, typename WeakPtrType>
	struct TImplicitWeakObjectPtr
	{
		TImplicitWeakObjectPtr(T* Ptr)
			: WeakObjPtr(Ptr)
		{
		}

		UE_REWRITE operator T* () const
		{
			return this->WeakObjPtr.Get();
		}

		TWeakObjectPtr<T, WeakPtrType> WeakObjPtr;
	};

	// This trait defines how different dynamic delegate payload arguments are captured.
	// If the trait is not specialized for a particular type, that type is not currently supported as a payload argument.
	template <typename WeakPtrType, typename T>
	struct TDynamicDelegateParamPayloadCaptureType;

	// Reference and cv-qualified arguments are captured by value and unqualified.
	template <typename WeakPtrType, typename T>
	struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, T&> : TDynamicDelegateParamPayloadCaptureType<WeakPtrType, T>
	{
	};
	template <typename WeakPtrType, typename T>
	struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, const T&> : TDynamicDelegateParamPayloadCaptureType<WeakPtrType, T>
	{
	};
	template <typename WeakPtrType, typename T>
	struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, const T> : TDynamicDelegateParamPayloadCaptureType<WeakPtrType, T>
	{
	};

	// Allowed:
	// - UObject pointers.
	// - TWeakObjectPtr.
	// - Numeric types.
	// - void (only for return types, not parameters, but C++ will stop you having a void parameter before getting here)
	// - Strings and FNames.
	template <typename WeakPtrType, UE::CUObject T>       struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, T*>          { using Type = TImplicitWeakObjectPtr<T, WeakPtrType>; };
	template <typename WeakPtrType, UE::CArithmetic T>    struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, T>           { using Type = T; };
	template <typename WeakPtrType, CWeakPointerish T>    struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, T>           { using Type = T; };
	template <typename WeakPtrType>                       struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, void>        { using Type = void; };
	template <typename WeakPtrType>                       struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, FAnsiString> { using Type = FAnsiString; };
	template <typename WeakPtrType>                       struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, FString>     { using Type = FString; };
	template <typename WeakPtrType>                       struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, FUtf8String> { using Type = FUtf8String; };
	template <typename WeakPtrType>                       struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, FName>       { using Type = FName; };
	template <typename WeakPtrType>                       struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, FText>       { using Type = FText; };
	template <typename WeakPtrType, UE::CUScriptStruct T> struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, T>           { using Type = T; };

	template <typename WeakPtrType, UE::CUEnum T>
	struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, T>
	{
#if UE_DYNAMIC_DELEGATE_UNSCOPED_ENUM_HACK
		using Type = std::conditional_t<UE::CUnscopedEnum<T>, TEnumAsByte<T>, T>;
#else
		using Type = T;
#endif
	};

	template <typename WeakPtrType, typename T>
	struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, TOptional<T>>
	{
		using Type = TOptional<T>;

		// If you get a compile error here, it's because you're trying to use an optional of an unsupported payload type
		using Validation = typename TDynamicDelegateParamPayloadCaptureType<WeakPtrType, T>::Type;
	};
	template <typename WeakPtrType, typename T>
	struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, TArray<T>>
	{
		using Type = TArray<T>;

		// If you get a compile error here, it's because you're trying to use an array of an unsupported payload type
		using Validation = typename TDynamicDelegateParamPayloadCaptureType<WeakPtrType, T>::Type;
	};
	template <typename WeakPtrType, typename T>
	struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, TSet<T>>
	{
		using Type = TSet<T>;

		// If you get a compile error here, it's because you're trying to use a set of an unsupported payload type
		using Validation = typename TDynamicDelegateParamPayloadCaptureType<WeakPtrType, T>::Type;
	};
	template <typename WeakPtrType, typename KeyType, typename ValueType>
	struct TDynamicDelegateParamPayloadCaptureType<WeakPtrType, TMap<KeyType, ValueType>>
	{
		using Type = TMap<KeyType, ValueType>;

		// If you get a compile error here, it's because you're trying to use a map with keys or values of an unsupported payload type
		using KeyValidation   = typename TDynamicDelegateParamPayloadCaptureType<WeakPtrType, KeyType>::Type;
		using ValueValidation = typename TDynamicDelegateParamPayloadCaptureType<WeakPtrType, ValueType>::Type;
	};

	template <typename WeakPtrType, typename T>
	using TDynamicDelegateParamPayloadCaptureType_T = typename TDynamicDelegateParamPayloadCaptureType<WeakPtrType, T>::Type;

	template <typename FunctionType, typename TupleType>
	struct TDynamicDelegateHasExactPayloadType
	{
		static constexpr inline bool Value = false;
	};

	template <typename... ParamTypes, typename... VarTypes>
	struct TDynamicDelegateHasExactPayloadType<void(ParamTypes...), TTuple<VarTypes...>>
	{
		static constexpr inline bool Value = (std::is_same_v<std::decay_t<ParamTypes>, VarTypes> && ...);
	};

	template <typename FunctionType, typename TupleType>
	constexpr inline bool DynamicDelegatePayloadIsLayoutCompatible()
	{
		if constexpr (TFunctionArity_V<FunctionType> != TTupleArity<TupleType>::Value)
		{
			return false;
		}
		else
		{
			return TDynamicDelegateHasExactPayloadType<FunctionType, TupleType>::Value;
		}
	}

	// An interface to a dynamic delegate payload.  Type-erases payload operations.
	template <typename WeakPtrType>
	struct IDynamicDelegatePayload
	{
		// Calls ObjectPtr->ProcessEvent(Function, ParametersPlusPayload...) on ObjectPtr.
		virtual void ProcessEvent(WeakPtrType ObjectPtr, UFunction* Function, void* Parameters) const = 0;

		// Serializes the payload in binary form - only intended for saving, so const.
		// Payloads are always binary, even within structured text archives.
		virtual void Serialize(FArchive& Ar) const = 0;

		// Returns the CoreUObject-neutral schema describing the payload's field types.
		virtual const FDelegatePayloadSchema& GetSchema() const = 0;

		virtual ~IDynamicDelegatePayload() = 0;
	};

	template <typename WeakPtrType>
	UE_REWRITE IDynamicDelegatePayload<WeakPtrType>::~IDynamicDelegatePayload() = default;

	template <typename WeakPtrType, typename MemFnType, typename... VarTypes>
	struct TDynamicDelegatePayload;

	template <typename WeakPtrType, typename... ParamTypes, typename... VarTypes>
	struct TDynamicDelegatePayload<WeakPtrType, void(ParamTypes...), VarTypes...> : IDynamicDelegatePayload<WeakPtrType>
	{
		using VarsTupleType = TTuple<VarTypes...>;

		static_assert(sizeof...(VarTypes) != 0, "Allocating an object to hold an empty payload is wasteful");

		template <typename... ArgTypes>
		[[nodiscard]] UE_REWRITE explicit TDynamicDelegatePayload(ArgTypes&&... Args)
			: Payload((ArgTypes&&)Args...)
		{
		}

		void ProcessEvent(WeakPtrType ObjectPtr, UFunction* Function, void* Parameters) const override
		{
			// The incoming Parameters will point to a block of arguments which were passed to the dynamic delegate
			// invocation.  These will be combined with the payload in order to produce a new parameters struct to
			// be passed to the ProcessEvent call.

			// If the payload replaces every argument, just pass the tuple itself as it should be the same layout.
			if constexpr (DynamicDelegatePayloadIsLayoutCompatible<void(ParamTypes...), VarsTupleType>())
			{
				ObjectPtr.Get()->ProcessEvent(Function, (void*)&Payload);
			}
			// ... otherwise we need to combine Params and Payload into a single struct
			else
			{
#if UE_DYNAMIC_DELEGATE_UNSCOPED_ENUM_HACK
				using FullTupleType   = TTuple<std::conditional_t<UE::CUnscopedEnum<std::decay_t<ParamTypes>>, TEnumAsByte<std::decay_t<ParamTypes>>, std::decay_t<ParamTypes>>...>;
#else
				using FullTupleType   = TTuple<std::decay_t<ParamTypes>...>;
#endif
				using ParamsTupleType = TTuplePopBack_T<FullTupleType, sizeof...(VarTypes)>;

				((ParamsTupleType*)Parameters)->ApplyBefore(
					[&Payload = Payload, ObjectPtr, Function](auto&&... Params)
					{
						Payload.ApplyAfter(
							[&Payload = Payload, ObjectPtr, Function](auto&&... ParamsAndVars)
							{
								FullTupleType NewParameters(Forward<decltype(ParamsAndVars)>(ParamsAndVars)...);
								ObjectPtr.Get()->ProcessEvent(Function, &NewParameters);
							},
							Forward<decltype(Params)>(Params)...
						);
					}
				);
			}
		}

		void Serialize(FArchive& Ar) const override
		{
			// Function is const because it's only expected to be used for saving.  The Payload is mutable so this is ok.
			checkSlow(!Ar.IsLoading());

			// TODO - implement serialization.
		}

		const FDelegatePayloadSchema& GetSchema() const override
		{
			// Stub — TDynamicDelegatePayload is replaced by FDynamicDelegatePropertyPayload (CoreUObject) in Step 3,
			// which builds the full schema with parameter names from the UFunction at bind time.
			static const FDelegatePayloadSchema Schema;
			return Schema;
		}

	private:
		// Mutable because ProcessEvent takes a non-const param pointer
		mutable VarsTupleType Payload;
	};

	template <typename WeakPtrType, typename BoundFuncType, typename... VarTypes>
	UE_REWRITE TSharedPtr<const IDynamicDelegatePayload<WeakPtrType>> MakeDynamicDelegatePayload(VarTypes&&... Vars)
	{
		return MakeShared<TDynamicDelegatePayload<WeakPtrType, BoundFuncType, TDynamicDelegateParamPayloadCaptureType_T<WeakPtrType, VarTypes>...>>(Forward<VarTypes>(Vars)...);
	}

	/** Callback function pointers for serializing/deserializing delegate payloads.
	*  Null until CoreUObject registers them at module startup (Step 4).
	*  All payload serialization is implemented in CoreUObject — Core only holds
	*  an opaque IDynamicDelegatePayload handle without knowing the details.
	*  Payloads are always serialized in binary form, even within structured text
	*  archives (where they appear as a base-64 blob via FArchiveUObjectFromStructuredArchive). */
	using FDelegatePayloadSerializeFunc      = void(*)(FArchive& Ar, const IDynamicDelegatePayload<FWeakObjectPtr>& Payload);
	using FDelegatePayloadDeserializeFunc    = TSharedPtr<IDynamicDelegatePayload<FWeakObjectPtr>>(*)(FArchive& Ar);
	using FDelegatePayloadSerializeSlotFunc  = void(*)(FStructuredArchiveSlot Slot, const IDynamicDelegatePayload<FWeakObjectPtr>& Payload);
	using FDelegatePayloadDeserializeSlotFunc = TSharedPtr<IDynamicDelegatePayload<FWeakObjectPtr>>(*)(FStructuredArchiveSlot Slot);

	CORE_API extern FDelegatePayloadSerializeFunc       GDelegatePayloadSerializeFunc;
	CORE_API extern FDelegatePayloadDeserializeFunc     GDelegatePayloadDeserializeFunc;
	CORE_API extern FDelegatePayloadSerializeSlotFunc   GDelegatePayloadSerializeSlotFunc;
	CORE_API extern FDelegatePayloadDeserializeSlotFunc GDelegatePayloadDeserializeSlotFunc;
#endif

	// Describes a type which is constructible with a ForceInit argument.
	template <typename T>
	concept CForceInitable = UE::CCompleteType<T> && std::is_constructible_v<T, EForceInit>;

	// The code below effectively reimplements a minimal version of TTuple, tailored to the needs
	// of delegates' param structs passed to ProcessEvent.
	//
	// It distinguishes a return element from parameter elements, giving it a different member name for
	// easy access, forcing its default value with either value init or EForceInit depending on the type.
	// 
	// It also directly constructs the elements rather than default construct and copy assign, and uses
	// perfect forwarding so that moves happen instead of copies where possible.

	template <typename T>
	struct TDelegateWrapperReturnBase
	{
		using MemberType = std::remove_cv_t<std::remove_reference_t<T>>;

#if UE_DYNAMIC_DELEGATE_UNSCOPED_ENUM_HACK
		using RemappedMemberType = std::conditional_t<UE::CUnscopedEnum<MemberType>, TEnumAsByte<MemberType>, MemberType>;
		RemappedMemberType ReturnValue = {};
#else
		MemberType ReturnValue = {};
#endif
	};
	template <CForceInitable T>
	struct TDelegateWrapperReturnBase<T>
	{
		using MemberType = std::remove_cv_t<std::remove_reference_t<T>>;

#if UE_DYNAMIC_DELEGATE_UNSCOPED_ENUM_HACK
		using RemappedMemberType = std::conditional_t<UE::CUnscopedEnum<MemberType>, TEnumAsByte<MemberType>, MemberType>;
		RemappedMemberType ReturnValue = T(ForceInit);
#else
		MemberType ReturnValue = T(ForceInit);
#endif
	};

	template <typename T, uint32 Index>
	struct TDelegateWrapperParmBase
	{
		using MemberType = std::remove_cv_t<std::remove_reference_t<T>>;

#if UE_DYNAMIC_DELEGATE_UNSCOPED_ENUM_HACK
		using RemappedMemberType = std::conditional_t<UE::CUnscopedEnum<MemberType>, TEnumAsByte<MemberType>, MemberType>;
		RemappedMemberType Member;
#else
		MemberType Member;
#endif

		// This odd constraint is to allow selection of the correct MoveOut overload by specifying the
		// index when they are merged into the derived class's overload set.
		//
		// The argument is a const reference because we want it to compile when passed a const param,
		// but the move will only happen when the the original parameter was a non-const reference, so
		// it's fine.
		template <uint32 CallIndex>
			requires (Index == CallIndex)
		UE_REWRITE void MoveOut(const MemberType& Out)
		{
			if constexpr (std::is_reference_v<T> && !std::is_const_v<std::remove_reference_t<T>>)
			{
				const_cast<MemberType&>(Out) = MoveTemp(this->Member);
			}
		}
	};

	// This is the dedicated tuple type, but it takes a function type instead of a list of types,
	// and it requires a TIntegerSequence parameter, which is hidden inside TTuple's
	// implementation but provided explicitly here because this doesn't need to have the nicest API.
	template <typename FuncType, typename IndexSequenceType>
	struct TDelegateWrapperParms;

	template <typename RetType, typename... ArgTypes, uint32... Indices>
	struct TDelegateWrapperParms<RetType(ArgTypes...), TIntegerSequence<uint32, Indices...>> : TDelegateWrapperParmBase<ArgTypes, Indices>..., TDelegateWrapperReturnBase<RetType>
	{
		// Merge all parameters' MoveOut overloads into a common overload set.
		using TDelegateWrapperParmBase<ArgTypes, Indices>::MoveOut...;

		template <typename... ConstructorArgTypes>
		[[nodiscard]] UE_REWRITE TDelegateWrapperParms(ConstructorArgTypes&&... Args)
			: TDelegateWrapperParmBase<ArgTypes, Indices>{ Forward<ConstructorArgTypes>(Args) }...
		{
		}
	};

	template <typename... ArgTypes, uint32... Indices>
	struct TDelegateWrapperParms<void(ArgTypes...), TIntegerSequence<uint32, Indices...>> : TDelegateWrapperParmBase<ArgTypes, Indices>...
	{
		// Merge all parameters' MoveOut overloads into a common overload set.
		using TDelegateWrapperParmBase<ArgTypes, Indices>::MoveOut...;

		template <typename... ConstructorArgTypes>
		[[nodiscard]] UE_REWRITE TDelegateWrapperParms(ConstructorArgTypes&&... Args)
			: TDelegateWrapperParmBase<ArgTypes, Indices>{ Forward<ConstructorArgTypes>(Args) }...
		{
		}
	};

	template <typename T>
	struct TDelegateCallTraits
	{
		using Type = std::conditional_t<std::is_reference_v<T>, T, const T&>;
	};

	template <typename T>
		requires (!UE::CReference<T> || UE::CConst<std::remove_reference_t<T>>) && UE::CScalar<std::remove_reference_t<T>>
	struct TDelegateCallTraits<T>
	{
		using Type = std::remove_cv_t<std::remove_reference_t<T>>;
	};

	template <typename T>
	using TDelegateCallTraits_T = typename TDelegateCallTraits<T>::Type;

	template <typename RetType, typename... ParamTypes>
	struct TDelegateFunctionWrapper
	{
	private:
		template <typename DelegateType, uint32... Indices>
		static RetType CallWrapperImpl(const DelegateType& Delegate, TIntegerSequence<uint32, Indices...>, ParamTypes... Args)
		{
			using TupleType = TDelegateWrapperParms<RetType(ParamTypes...), TIntegerSequence<uint32, Indices...>>;

			TupleType Parms((ParamTypes&&)Args...);

			// TMulticastScriptDelegate::ProcessMulticastDelegate was renamed to ProcessDelegate to allow this template to be
			// be used for single cast and multicasts without needing a special construct to select the correct name.
			Delegate.template ProcessDelegate<UObject>(&Parms);

			// Copy out params only from the struct to the arguments.
			(Parms.template MoveOut<Indices>(Args), ...);

			// Copy return value from the struct if there is one.
			if constexpr (!std::is_void_v<RetType>)
			{
				// Make sure uint32 -> bool is supported
				if constexpr (std::is_same_v<const RetType, const bool>)
				{
					return !!Parms.ReturnValue;
				}
				else
				{
					return MoveTemp(Parms.ReturnValue);
				}
			}
		}

	public:
		template <typename ScriptDelegateType>
		UE_REWRITE static RetType CallWrapper(const ScriptDelegateType& Delegate, ParamTypes... Args)
		{
			return CallWrapperImpl(Delegate, TMakeIntegerSequence<uint32, sizeof...(ParamTypes)>{}, Args...);
		}
	};
}

/**
 * Script delegate base class.
 */
template <typename InThreadSafetyMode>
class TScriptDelegate : public TDelegateAccessHandlerBase<InThreadSafetyMode>
{
public:
	using ThreadSafetyMode = InThreadSafetyMode;
	using WeakPtrType = typename UE::Core::Private::TScriptDelegateTraits<InThreadSafetyMode>::WeakPtrType;

private:
	template <typename>
	friend class TScriptDelegate;

	template<typename>
	friend class TMulticastScriptDelegate;

	using Super = TDelegateAccessHandlerBase<ThreadSafetyMode>;
	using typename Super::FReadAccessScope;
	using Super::GetReadAccessScope;
	using typename Super::FWriteAccessScope;
	using Super::GetWriteAccessScope;

public:
	/** Default constructor. */
	TScriptDelegate() 
		: Object( nullptr )
		, FunctionName( NAME_None )
	{
	}

	TScriptDelegate(const TScriptDelegate& Other)
	{
		FReadAccessScope OtherReadScope = Other.GetReadAccessScope();

		Object = Other.Object;
		FunctionName = Other.FunctionName;
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
		Payload = Other.Payload;
#endif
	}

	// TODO: We could add move operations to avoid the refcount bump of payloads, but it
	// would require resetting the object and function name too.  This could break existing
	// code which is relying on execute-after-move.  We don't want to support that, but we
	// need a deprecation strategy.

	TScriptDelegate& operator=(const TScriptDelegate& Other)
	{
		WeakPtrType OtherObject;
		FName OtherFunctionName;
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
		TSharedPtr<const UE::Core::Private::IDynamicDelegatePayload<WeakPtrType>> OtherPayload;
#endif

		{
			FReadAccessScope OtherReadScope = Other.GetReadAccessScope();
			OtherObject = Other.Object;
			OtherFunctionName = Other.FunctionName;
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
			OtherPayload = Other.Payload;
#endif
		}

		{
			FWriteAccessScope WriteScope = GetWriteAccessScope();
			Object = OtherObject;
			FunctionName = OtherFunctionName;
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
			Payload = MoveTemp(OtherPayload);
#endif
		}

		return *this;
	}

private:

	template <class UObjectTemplate>
	inline bool IsBound_Internal() const
	{
		if (FunctionName != NAME_None)
		{
			if (UObject* ObjectPtr = Object.Get())
			{
				return ((UObjectTemplate*)ObjectPtr)->FindFunction(FunctionName) != nullptr;
			}
		}

		return false;
	}

public:

	/**
	 * Binds a UFunction to this delegate.
	 *
	 * @param InObject The object to call the function on.
	 * @param InFunctionName The name of the function to call.
	 */
	void BindUFunction( UObject* InObject, const FName& InFunctionName )
	{
		FWriteAccessScope WriteScope = GetWriteAccessScope();

		Object = InObject;
		FunctionName = InFunctionName;
		// TODO: Add payload support?  We don't have the dynamic delegate signature here to create a payload.
	}

	/** 
	 * Checks to see if the user object bound to this delegate is still valid
	 *
	 * @return  True if the object is still valid and it's safe to execute the function call
	 */
	inline bool IsBound() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();
		return IsBound_Internal<UObject>();
	}

	/** 
	 * Checks to see if this delegate is bound to the given user object.
	 *
	 * @return	True if this delegate is bound to InUserObject, false otherwise.
	 */
	inline bool IsBoundToObject(void const* InUserObject) const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();
		return InUserObject && (InUserObject == GetUObject());
	}

	/** 
	 * Checks to see if this delegate is bound to the given user object, even if the object is unreachable.
	 *
	 * @return	True if this delegate is bound to InUserObject, false otherwise.
	 */
	bool IsBoundToObjectEvenIfUnreachable(void const* InUserObject) const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();
		return InUserObject && InUserObject == GetUObjectEvenIfUnreachable();
	}

	/** 
	 * Checks to see if the user object bound to this delegate will ever be valid again
	 *
	 * @return  True if the object is still valid and it's safe to execute the function call
	 */
	inline bool IsCompactable() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();
		return FunctionName == NAME_None || 
#if UE_WITH_REMOTE_OBJECT_HANDLE
			!Object.IsValid(true);
#else
			!Object.Get(true);
#endif
	}

	/**
	 * Unbinds this delegate
	 */
	void Unbind()
	{
		FWriteAccessScope WriteScope = GetWriteAccessScope();

		Object = nullptr;
		FunctionName = NAME_None;
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
		Payload = nullptr;
#endif
	}

	/**
	 * Unbinds this delegate (another name to provide a similar interface to TMulticastScriptDelegate)
	 */
	void Clear()
	{
		Unbind();
	}

	/**
	 * Converts this delegate to a string representation
	 *
	 * @return	Delegate in string format
	 */
	template <class UObjectTemplate>
	inline FString ToString() const
	{
		if( IsBound() )
		{
			FReadAccessScope ReadScope = GetReadAccessScope();
			// TODO: Stringify payloads?
			return ((UObjectTemplate*)GetUObject())->GetPathName() + TEXT(".") + GetFunctionName().ToString();
		}
		return TEXT( "<Unbound>" );
	}

	/** Delegate serialization */
	friend FArchive& operator<<( FArchive& Ar, TScriptDelegate& D )
	{
		FReadAccessScope ReadScope = D.GetReadAccessScope();

		Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

		// The original implementation serialized a bool to represent the presence of a payload.
		// The new implementation folds the flag into the function name so that the binary layout is unaffected for dynamic delegates without payloads
		bool bSerializePayloadData         = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::DynamicDelegatePayloads;
		bool bPayloadDataUsesCompactFormat = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::CompactDynamicDelegatePayloads;
		if (!bPayloadDataUsesCompactFormat)
		{
			Ar << D.Object << D.FunctionName;
		}

		bool bHasPayload = false;
		if (bSerializePayloadData)
		{
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
			bHasPayload = !!D.Payload;
#endif
			if (!bPayloadDataUsesCompactFormat)
			{
				Ar << bHasPayload;
			}
		}

		if (bPayloadDataUsesCompactFormat)
		{
			FName FunctionNameWithOptionalPayloadFlag = D.FunctionName;
			if (Ar.IsSaving())
			{
				if (bHasPayload)
				{
					FunctionNameWithOptionalPayloadFlag = *(FunctionNameWithOptionalPayloadFlag.ToString() + UE_DYNAMIC_DELEGATE_PAYLOAD_FUNCTION_NAME_SUFFIX);
				}
			}

			Ar << D.Object << FunctionNameWithOptionalPayloadFlag;

			if (Ar.IsLoading())
			{
				FString FunctionNameWithOptionalPayloadFlagStr = FunctionNameWithOptionalPayloadFlag.ToString();
				bHasPayload = FunctionNameWithOptionalPayloadFlagStr.RemoveFromEnd(UE_DYNAMIC_DELEGATE_PAYLOAD_FUNCTION_NAME_SUFFIX);
				D.FunctionName = *FunctionNameWithOptionalPayloadFlagStr;
			}
		}

		if (bHasPayload)
		{
			if (Ar.IsLoading())
			{
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
				if (ensureMsgf(UE::Core::Private::GDelegatePayloadDeserializeFunc, TEXT("Trying to deserialize a dynamic delegate payload without CoreUObject's deserialize factory being registered")))
				{
					D.Payload = UE::Core::Private::GDelegatePayloadDeserializeFunc(Ar);
				}
				else
				{
					D.Unbind();
					// TODO: drain payload bytes once FDynamicDelegatePropertyPayload exists (Step 3)
				}
#else
				ensureMsgf(false, TEXT("Trying to deserialize a dynamic delegate payload when payloads are disabled"));
				D.Unbind();
				// TODO: drain bytes (Step 3)
#endif
			}
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
			else
			{
				UE::Core::Private::GDelegatePayloadSerializeFunc(Ar, *D.Payload);
			}
#endif
		}

		return Ar;
	}

	/** Delegate serialization */
	friend void operator<<(FStructuredArchive::FSlot Slot, TScriptDelegate& D)
	{
		FReadAccessScope ReadScope = D.GetReadAccessScope();

		FArchive& UnderlyingAr = Slot.GetUnderlyingArchive();

		UnderlyingAr.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

		bool bSerializePayloadData         = UnderlyingAr.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::DynamicDelegatePayloads;
		bool bPayloadDataUsesCompactFormat = UnderlyingAr.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::CompactDynamicDelegatePayloads;

		FStructuredArchive::FRecord Record = Slot.EnterRecord();

		bool bHasPayload = false;
		if (bSerializePayloadData)
		{
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
			bHasPayload = !!D.Payload;
#endif
		}

		// For text-format archives, payload presence is expressed structurally via TryEnterField
		// and the function name is always clean.  For binary-format archives, we use the function
		// name suffix trick (v263+) to avoid shifting binary data.
		if (bPayloadDataUsesCompactFormat && !UnderlyingAr.IsTextFormat())
		{
			// Binary structured path — same suffix approach as the binary operator<<
			FName FunctionNameWithOptionalPayloadFlag = D.FunctionName;
			if (UnderlyingAr.IsSaving())
			{
				if (bHasPayload)
				{
					FunctionNameWithOptionalPayloadFlag = *(FunctionNameWithOptionalPayloadFlag.ToString() + UE_DYNAMIC_DELEGATE_PAYLOAD_FUNCTION_NAME_SUFFIX);
				}
			}

			Record << SA_VALUE(TEXT("Object"), D.Object) << SA_VALUE(TEXT("FunctionName"), FunctionNameWithOptionalPayloadFlag);

			if (UnderlyingAr.IsLoading())
			{
				FString FunctionNameWithOptionalPayloadFlagStr = FunctionNameWithOptionalPayloadFlag.ToString();
				bHasPayload = FunctionNameWithOptionalPayloadFlagStr.RemoveFromEnd(UE_DYNAMIC_DELEGATE_PAYLOAD_FUNCTION_NAME_SUFFIX);
				D.FunctionName = *FunctionNameWithOptionalPayloadFlagStr;
			}
		}
		else
		{
			// Text format or pre-v263: clean function name, TryEnterField for payload presence
			Record << SA_VALUE(TEXT("Object"), D.Object) << SA_VALUE(TEXT("FunctionName"), D.FunctionName);
		}

		// Serialize payload data
		auto SerializePayload = [&UnderlyingAr, &D](FStructuredArchiveSlot PayloadSlot)
		{
			if (UnderlyingAr.IsLoading())
			{
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
				if (ensureMsgf(UE::Core::Private::GDelegatePayloadDeserializeSlotFunc, TEXT("Trying to deserialize a dynamic delegate payload without CoreUObject's deserialize factory being registered")))
				{
					D.Payload = UE::Core::Private::GDelegatePayloadDeserializeSlotFunc(PayloadSlot);
				}
				else
				{
					D.Unbind();
					// TODO: drain payload bytes once FDynamicDelegatePropertyPayload exists (Step 3)
				}
#else
				ensureMsgf(false, TEXT("Trying to deserialize a dynamic delegate payload when payloads are disabled"));
				D.Unbind();
				// TODO: drain bytes (Step 3)
#endif
			}
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
			else
			{
				UE::Core::Private::GDelegatePayloadSerializeSlotFunc(PayloadSlot, *D.Payload);
			}
#endif
		};

		if (bSerializePayloadData)
		{
			if (bPayloadDataUsesCompactFormat && !UnderlyingAr.IsTextFormat())
			{
				// Binary v263+: bHasPayload already determined from suffix
				if (bHasPayload)
				{
					SerializePayload(Record.EnterField(TEXT("Payload")));
				}
			}
			else
			{
				// Text format or v262: TryEnterField expresses presence structurally
				if (TOptional<FStructuredArchiveSlot> PayloadSlot = Record.TryEnterField(TEXT("Payload"), bHasPayload))
				{
					SerializePayload(*PayloadSlot);
				}
			}
		}
	}

	/** Comparison operators */
	inline bool operator==( const TScriptDelegate& Other ) const
	{
		WeakPtrType OtherObject;
		FName OtherFunctionName;

		{
			FReadAccessScope OtherReadScope = Other.GetReadAccessScope();
			OtherObject = Other.Object;
			OtherFunctionName = Other.FunctionName;
		}

		bool bResult;

		{
			FReadAccessScope ThisReadScope = GetReadAccessScope();
			bResult = Object == OtherObject && FunctionName == OtherFunctionName;
			// TODO: Compare payloads
		}

		return bResult;
	}

	UE_FORCEINLINE_HINT bool operator!=( const TScriptDelegate& Other ) const
	{
		return !operator==(Other);
	}

	/** 
	 * Gets the object bound to this delegate
	 *
	 * @return	The object
	 */
	UObject* GetUObject()
	{
		FWriteAccessScope WriteScope = GetWriteAccessScope();

		// Downcast UObjectBase to UObject
		return static_cast< UObject* >( Object.Get() );
	}

	/**
	 * Gets the object bound to this delegate (const)
	 *
	 * @return	The object
	 */
	const UObject* GetUObject() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		// Downcast UObjectBase to UObject
		return static_cast< const UObject* >( Object.Get() );
	}

	/** 
	 * Gets the object bound to this delegate, even if the object is unreachable
	 *
	 * @return	The object
	 */
	UObject* GetUObjectEvenIfUnreachable()
	{
		FWriteAccessScope WriteScope = GetWriteAccessScope();

		// Downcast UObjectBase to UObject
		return static_cast< UObject* >( Object.GetEvenIfUnreachable() );
	}

	/**
	 * Gets the object bound to this delegate (const), even if the object is unreachable
	 *
	 * @return	The object
	 */
	const UObject* GetUObjectEvenIfUnreachable() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		// Downcast UObjectBase to UObject
		return static_cast< const UObject* >( Object.GetEvenIfUnreachable() );
	}

	UE_DEPRECATED(5.8, "GetUObjectRef has been deprecated.")
	WeakPtrType& GetUObjectRef()
	{
		return Object;
	}

	UE_DEPRECATED(5.8, "GetUObjectRef has been deprecated.")
	const WeakPtrType& GetUObjectRef() const
	{
		return Object;
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	auto GetRemoteId() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		return Object.GetRemoteId();
	}
#endif

	/**
	 * Gets the name of the function to call on the bound object
	 *
	 * @return	Function name
	 */
	FName GetFunctionName() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();
		return FunctionName;
	}

	/**
	 * Executes a delegate by calling the named function on the object bound to the delegate.  You should
	 * always first verify that the delegate is safe to execute by calling IsBound() before calling this function.
	 * In general, you should never call this function directly.  Instead, call Execute() on a derived class.
	 *
	 * @param	Parameters		Parameter structure
	 */
	//CORE_API void ProcessDelegate(void* Parameters) const;
	template <class UObjectTemplate>
	void ProcessDelegate( void* Parameters ) const
	{
		UObjectTemplate* ObjectPtr;
		UFunction* Function;

		{	// to avoid MT access check if the delegate is deleted from inside of its callback, we don't cover the callback execution
			// by access protection scope
			// the `const` on the method is a lie
			FWriteAccessScope WriteScope = const_cast<TScriptDelegate*>(this)->GetWriteAccessScope();

			checkf( Object.IsValid() != false, TEXT( "ProcessDelegate() called with no object bound to delegate!" ) );
			checkf( FunctionName != NAME_None, TEXT( "ProcessDelegate() called with no function name set!" ) );

			// Object was pending kill, so we cannot execute the delegate.  Note that it's important to assert
			// here and not simply continue execution, as memory may be left uninitialized if the delegate is
			// not able to execute, resulting in much harder-to-detect code errors.  Users should always make
			// sure IsBound() returns true before calling ProcessDelegate()!
			ObjectPtr = static_cast<UObjectTemplate*>(Object.Get());	// Down-cast
			checkSlow( IsValid(ObjectPtr) );

			// Object *must* implement the specified function
			Function = ObjectPtr->FindFunctionChecked(FunctionName);
		}

		// Execute the delegate!
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
		if (this->Payload)
		{
			Payload->ProcessEvent(ObjectPtr, Function, Parameters);
		}
		else
#endif
		{
			ObjectPtr->ProcessEvent(Function, Parameters);
		}
	}

	[[nodiscard]] friend uint32 GetTypeHash(const TScriptDelegate& Delegate)
	{
		FReadAccessScope ReadScope = Delegate.GetReadAccessScope();

		// TODO: Hash payloads
		return HashCombine(GetTypeHash(Delegate.Object), GetTypeHash(Delegate.GetFunctionName()));
	}

	template<typename OtherThreadSafetyMode>
	static TScriptDelegate CopyFrom(const TScriptDelegate<OtherThreadSafetyMode>& Other)
	{
		static_assert(std::is_same_v<ThreadSafetyMode, typename UE::Core::Private::TScriptDelegateTraits<ThreadSafetyMode>::UnicastThreadSafetyModeForMulticasts>);

		typename TScriptDelegate<OtherThreadSafetyMode>::FReadAccessScope OtherReadScope = Other.GetReadAccessScope();

		TScriptDelegate Copy;
		Copy.Object = Other.Object;
		Copy.FunctionName = Other.FunctionName;
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
		Copy.Payload = Other.Payload;
#endif

		return Copy;
	}

protected:
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
	template <typename BoundFuncType, typename... VarTypes>
	UE_REWRITE void BindProtected(WeakPtrType InObject, FName InFunctionName, VarTypes&&... Vars)
	{
		this->Object = InObject;
		this->FunctionName = InFunctionName;
		if constexpr (sizeof...(VarTypes) != 0)
		{
			this->Payload = UE::Core::Private::MakeDynamicDelegatePayload<WeakPtrType, BoundFuncType>(Forward<VarTypes>(Vars)...);
		}
	}
#else
	UE_REWRITE void BindProtected(WeakPtrType InObject, FName InFunctionName)
	{
		this->Object = InObject;
		this->FunctionName = InFunctionName;
	}
#endif

private:
	/** The object bound to this delegate, or nullptr if no object is bound */
	WeakPtrType Object;

	/** Name of the function to call on the bound object */
	FName FunctionName;
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS
	TSharedPtr<const UE::Core::Private::IDynamicDelegatePayload<WeakPtrType>> Payload;
#endif

	friend struct TIsZeroConstructType<TScriptDelegate>;
};

template<typename ThreadSafetyMode>
struct TIsZeroConstructType<TScriptDelegate<ThreadSafetyMode>>
{
	static constexpr bool Value = 
		TIsZeroConstructType<typename UE::Core::Private::TScriptDelegateTraits<ThreadSafetyMode>::WeakPtrType>::Value &&
		TIsZeroConstructType<typename TScriptDelegate<ThreadSafetyMode>::Super>::Value;
};

/**
 * Script multi-cast delegate base class
 */
template <typename InThreadSafetyMode>
class TMulticastScriptDelegate : public TDelegateAccessHandlerBase<InThreadSafetyMode>
{
private:
	using Super = TDelegateAccessHandlerBase<InThreadSafetyMode>;
	using typename Super::FReadAccessScope;
	using Super::GetReadAccessScope;
	using typename Super::FWriteAccessScope;
	using Super::GetWriteAccessScope;

	using UnicastDelegateType = TScriptDelegate<typename UE::Core::Private::TScriptDelegateTraits<InThreadSafetyMode>::UnicastThreadSafetyModeForMulticasts>;

public:
	using ThreadSafetyMode = typename UE::Core::Private::TScriptDelegateTraits<InThreadSafetyMode>::ThreadSafetyMode;
	using InvocationListType = TArray<UnicastDelegateType>;

	TMulticastScriptDelegate() = default;

	TMulticastScriptDelegate(const TMulticastScriptDelegate& Other)
	{
		InvocationListType LocalCopy;

		{
			FReadAccessScope OtherReadScope = Other.GetReadAccessScope();
			LocalCopy = Other.InvocationList;
		}

		InvocationList = MoveTemp(LocalCopy);
	}

	TMulticastScriptDelegate& operator=(const TMulticastScriptDelegate& Other)
	{
		InvocationListType LocalCopy;
		{
			FReadAccessScope OtherReadScope = Other.GetReadAccessScope();
			LocalCopy = Other.InvocationList;
		}

		{
			FWriteAccessScope WriteScope = GetWriteAccessScope();
			InvocationList = MoveTemp(LocalCopy);
		}

		return *this;
	}

	TMulticastScriptDelegate(TMulticastScriptDelegate&& Other)
	{
		InvocationListType LocalStorage;

		{
			FReadAccessScope OtherReadScope = Other.GetReadAccessScope();
			LocalStorage = MoveTemp(Other.InvocationList);
		}

		InvocationList = MoveTemp(LocalStorage);
	}

	TMulticastScriptDelegate& operator=(TMulticastScriptDelegate&& Other)
	{
		InvocationListType LocalStorage;

		{
			FReadAccessScope OtherReadScope = Other.GetReadAccessScope();
			LocalStorage = MoveTemp(Other.InvocationList);
		}

		{
			FWriteAccessScope WriteScope = GetWriteAccessScope();
			InvocationList = MoveTemp(LocalStorage);
		}

		return *this;
	}

public:

	/**
	 * Checks to see if any functions are bound to this multi-cast delegate
	 *
	 * @return	True if any functions are bound
	 */
	inline bool IsBound() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		return InvocationList.Num() > 0;
	}

	/**
	 * Checks whether a function delegate is already a member of this multi-cast delegate's invocation list
	 *
	 * @param	InDelegate	Delegate to check
	 * @return	True if the delegate is already in the list.
	 */
	bool Contains( const TScriptDelegate<ThreadSafetyMode>& InDelegate ) const
	{
		const UObject* Object;
		FName FunctionName;

		{
			FReadAccessScope OtherReadScope = InDelegate.GetReadAccessScope();
			Object = InDelegate.Object.Get();
			FunctionName = InDelegate.FunctionName;
			// TODO: Compare payloads
		}

		return Contains(Object, FunctionName);
	}

	/**
	 * Checks whether a function delegate is already a member of this multi-cast delegate's invocation list
	 *
	 * @param	InObject		Object of the delegate to check
	 * @param	InFunctionName	Function name of the delegate to check
	 * @return	True if the delegate is already in the list.
	 */
	bool Contains( const UObject* InObject, FName InFunctionName ) const
	{
		// TODO: Accept payloads

		FReadAccessScope ReadScope = GetReadAccessScope();

		return InvocationList.ContainsByPredicate( [=]( const UnicastDelegateType& Delegate ){
			return Delegate.GetFunctionName() == InFunctionName && Delegate.IsBoundToObjectEvenIfUnreachable(InObject);
		} );
	}

	/**
	 * Adds a function delegate to this multi-cast delegate's invocation list
	 *
	 * @param	InDelegate	Delegate to add
	 */
	void Add( const TScriptDelegate<ThreadSafetyMode>& InDelegate )
	{
		UnicastDelegateType LocalCopy = UnicastDelegateType::CopyFrom(InDelegate);

		{
			FWriteAccessScope WriteScope = GetWriteAccessScope();

			// First check for any objects that may have expired
			CompactInvocationList();

			// Add the delegate
			AddInternal(MoveTemp(LocalCopy));
		}
	}

	/**
	 * Adds a function delegate to this multi-cast delegate's invocation list if a delegate with the same signature
	 * doesn't already exist in the invocation list
	 *
	 * @param	InDelegate	Delegate to add
	 */
	void AddUnique( const TScriptDelegate<ThreadSafetyMode>& InDelegate )
	{
		UnicastDelegateType LocalCopy = UnicastDelegateType::CopyFrom(InDelegate);

		{
			FWriteAccessScope WriteScope = GetWriteAccessScope();

			// Add the delegate, if possible
			AddUniqueInternal(MoveTemp(LocalCopy));

			// Then check for any objects that may have expired
			CompactInvocationList();
		}
	}

	/**
	 * Removes a function from this multi-cast delegate's invocation list (performance is O(N)).  Note that the
	 * order of the delegates may not be preserved!
	 *
	 * @param	InDelegate	Delegate to remove
	 */
	void Remove( const TScriptDelegate<ThreadSafetyMode>& InDelegate )
	{
		UnicastDelegateType LocalCopy = UnicastDelegateType::CopyFrom(InDelegate);

		{
			FWriteAccessScope WriteScope = GetWriteAccessScope();

			// Remove the delegate
			RemoveInternal(LocalCopy);

			// Check for any delegates that may have expired
			CompactInvocationList();
		}
	}

	/**
	 * Removes a function from this multi-cast delegate's invocation list (performance is O(N)).  Note that the
	 * order of the delegates may not be preserved!
	 *
	 * @param	InObject		Object of the delegate to remove
	 * @param	InFunctionName	Function name of the delegate to remove
	 */
	void Remove( const UObject* InObject, FName InFunctionName )
	{
		// TODO: Accept payloads

		FWriteAccessScope WriteScope = GetWriteAccessScope();

		// Remove the delegate
		RemoveInternal( InObject, InFunctionName );

		// Check for any delegates that may have expired
		CompactInvocationList();
	}

	/**
	 * Removes all delegate bindings from this multicast delegate's
	 * invocation list that are bound to the specified object.
	 *
	 * This method also compacts the invocation list.
	 *
	 * @param InObject The object to remove bindings for.
	 */
	void RemoveAll(const UObject* Object)
	{
		FWriteAccessScope WriteScope = GetWriteAccessScope();

		for (int32 BindingIndex = InvocationList.Num() - 1; BindingIndex >= 0; --BindingIndex)
		{
			const UnicastDelegateType& Binding = InvocationList[BindingIndex];

			if (Binding.IsBoundToObject(Object) || Binding.IsCompactable())
			{
				InvocationList.RemoveAtSwap(BindingIndex);
			}
		}
	}

	/**
	 * Removes all functions from this delegate's invocation list
	 */
	void Clear()
	{
		FWriteAccessScope WriteScope = GetWriteAccessScope();
		InvocationList.Empty();
	}

	/**
	 * Converts this delegate to a string representation
	 *
	 * @return	Delegate in string format
	 */
	template <typename UObjectTemplate>
	inline FString ToString() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		if( IsBound() )
		{
			FString AllDelegatesString = "[";
			bool bAddComma = false;
			for( typename InvocationListType::TConstIterator CurDelegate( InvocationList ); CurDelegate; ++CurDelegate )
			{
				if (bAddComma)
				{
					AllDelegatesString += TEXT( ", " );
				}
				bAddComma = true;
				AllDelegatesString += CurDelegate->template ToString<UObjectTemplate>();
			}
			AllDelegatesString += TEXT( "]" );
			return AllDelegatesString;
		}
		return TEXT( "<Unbound>" );
	}

	/** Multi-cast delegate serialization */
	friend FArchive& operator<<(FArchive& Ar, TMulticastScriptDelegate& D)
	{
		// Note that !IsSaving is not the same as IsLoading. See, e.g., FArchiveReplaceObjectRef
		if (!Ar.IsSaving())
		{
			FWriteAccessScope WriteScope = D.GetWriteAccessScope();
			Ar << D.InvocationList;
			// After loading the delegate, clean up the list to make sure there are no bad object references
			// unless the archive is used to migrate remote objects in which case we don't want to resolve weak object pointers
			// which may be pointing to objects that haven't been migrated yet
			if (Ar.IsLoading() 
#if UE_WITH_REMOTE_OBJECT_HANDLE
				&& !Ar.HasAnyPortFlags(PPF_AvoidRemoteObjectMigration)
#endif
				)
			{
				D.CompactInvocationList();
			}
		}
		else
		{
			FReadAccessScope ReadScope = D.GetReadAccessScope();
			// When saving the delegate, clean up the list to make sure there are no bad object references
			// Don't do this in place because we don't want to require a write lock
			typedef TArray<UnicastDelegateType, TInlineAllocator<4>> FInlineInvocationList;
			FInlineInvocationList CompactedList;
			for (const UnicastDelegateType& Delegate : D.InvocationList)
			{
				if (!Delegate.IsCompactable())
				{
					CompactedList.Add(Delegate);
				}
			}
			Ar << CompactedList;
		}
		return Ar;
	}

	friend void operator<<(FStructuredArchive::FSlot Slot, TMulticastScriptDelegate& D)
	{
		FWriteAccessScope WriteScope = D.GetWriteAccessScope();

		FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

		if (UnderlyingArchive.IsSaving())
		{
			// When saving the delegate, clean up the list to make sure there are no bad object references
			D.CompactInvocationList();
		}

		Slot << D.InvocationList;

		if (UnderlyingArchive.IsLoading())
		{
			// After loading the delegate, clean up the list to make sure there are no bad object references
			D.CompactInvocationList();
		}
	}

	/**
	 * Executes a multi-cast delegate by calling all functions on objects bound to the delegate.  Always
	 * safe to call, even if when no objects are bound, or if objects have expired.  In general, you should
	 * never call this function directly.  Instead, call Broadcast() on a derived class. Note that this function
	 * is not truly const because it will clean up any compactable entries in the invocation list.
	 *
	 * @param	Params				Parameter structure
	 */
	template <class UObjectTemplate>
	void ProcessDelegate(void* Parameters) const
	{
		{
			FReadAccessScope ReadScope = const_cast<TMulticastScriptDelegate*>(this)->GetReadAccessScope();

			if( InvocationList.Num() > 0 )
			{
				// Create a copy of the invocation list, just in case the list is modified by one of the callbacks during the broadcast
				typedef TArray< UnicastDelegateType, TInlineAllocator< 4 > > FInlineInvocationList;
				FInlineInvocationList InvocationListCopy = FInlineInvocationList(InvocationList);

				// Invoke each bound function
				for( typename FInlineInvocationList::TConstIterator FunctionIt( InvocationListCopy ); FunctionIt; ++FunctionIt )
				{
					if( FunctionIt->IsBound() )
					{
						// Invoke this delegate!
						FunctionIt->template ProcessDelegate<UObjectTemplate>(Parameters);
					}
				}
			}
		}

		{
			FWriteAccessScope WriteScope = const_cast<TMulticastScriptDelegate*>(this)->GetWriteAccessScope();
			// Removes need to occur under a separate write scope because we don't want to hold a write lock during execution
			// We want to take the least restrictive lock (guard, really) possible in order to permit the callbacks to
			// inspect the multicast delegate itself (e.g., when using the serialization system to inspect/explore data or find references)
			CompactInvocationList();
		}
	}

	template <class UObjectTemplate>
	UE_DEPRECATED(5.8, "ProcessMulticastDelegate has been deprecated, please use ProcessDelegate instead.")
	UE_REWRITE void ProcessMulticastDelegate(void* Parameters) const
	{
		ProcessDelegate(Parameters);
	}

	/**
	 * Returns all objects associated with this multicast-delegate.
	 * For advanced uses only -- you should never need call this function in normal circumstances.
 	 * @return	List of objects bound to this delegate
	 */
	TArray< UObject* > GetAllObjects() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();

		TArray< UObject* > OutputList;
		for( typename InvocationListType::TIterator CurDelegate( InvocationList ); CurDelegate; ++CurDelegate )
		{
			UObject* CurObject = CurDelegate->GetUObject();
			if( CurObject != nullptr )
			{
				OutputList.Add( CurObject );
			}
		}
		return OutputList;
	}

	/**
	 * Returns all objects associated with this multicast-delegate, even if unreachable.
	 * For advanced uses only -- you should never need call this function in normal circumstances.
	 * @return	List of objects bound to this delegate
	 */
	TArray<UObject*> GetAllObjectsEvenIfUnreachable() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();
		using WeakPtrType = typename UnicastDelegateType::WeakPtrType;
		TArray<UObject*> Result;
		for (typename InvocationListType::TIterator CurDelegate(InvocationList); CurDelegate; ++CurDelegate)
		{
			if (UObject* CurObjectPtr = CurDelegate->GetUObjectEvenIfUnreachable())
			{
				Result.Add(CurObjectPtr);
			}
		}
		return Result;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "GetAllObjectRefsEvenIfUnreachable has been deprecated.")
	TArray< typename UnicastDelegateType::WeakPtrType* > GetAllObjectRefsEvenIfUnreachable() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();
		using WeakPtrType = typename UnicastDelegateType::WeakPtrType;
		TArray< WeakPtrType* > OutputList;
		for( typename InvocationListType::TIterator CurDelegate( InvocationList ); CurDelegate; ++CurDelegate )
		{
			WeakPtrType& CurObject = CurDelegate->GetUObjectRef();
			if( CurObject.GetEvenIfUnreachable() != nullptr )
			{
				OutputList.Add( &CurObject );
			}
		}
		return OutputList;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Returns the amount of memory allocated by this delegate's invocation list, not including sizeof(*this).
	 */
	SIZE_T GetAllocatedSize() const
	{
		FReadAccessScope ReadScope = GetReadAccessScope();
		return InvocationList.GetAllocatedSize();
	}

protected:

	/**
	 * Adds a function delegate to this multi-cast delegate's invocation list
	 *
	 * @param	InDelegate	Delegate to add
	 */
	void AddInternal(UnicastDelegateType&& InDelegate)
	{
#if DO_ENSURE
		// Verify same function isn't already bound
		const int32 NumFunctions = InvocationList.Num();
		for( int32 CurFunctionIndex = 0; CurFunctionIndex < NumFunctions; ++CurFunctionIndex )
		{
			(void)ensure( InvocationList[ CurFunctionIndex ] != InDelegate );
		}
#endif // DO_CHECK
		InvocationList.Add(MoveTemp(InDelegate));
	}

	/**
	 * Adds a function delegate to this multi-cast delegate's invocation list, if a delegate with that signature
	 * doesn't already exist
	 *
	 * @param	InDelegate	Delegate to add
	 */
	void AddUniqueInternal(UnicastDelegateType&& InDelegate)
	{
		// Add the item to the invocation list only if it is unique
		InvocationList.AddUnique(MoveTemp(InDelegate));
	}

	/**
	 * Removes a function from this multi-cast delegate's invocation list (performance is O(N)).  Note that the
	 * order of the delegates may not be preserved!
	 *
	 * @param	InDelegate	Delegate to remove
	 */
	void RemoveInternal( const UnicastDelegateType& InDelegate ) const
	{
		InvocationList.RemoveSingleSwap(InDelegate);
	}

	/**
	 * Removes a function from this multi-cast delegate's invocation list (performance is O(N)).  Note that the
	 * order of the delegates may not be preserved!
	 *
	 * @param	InObject		Object of the delegate to remove
	 * @param	InFunctionName	Function name of the delegate to remove
	 */
	void RemoveInternal( const UObject* InObject, FName InFunctionName ) const
	{
		int32 FoundDelegate = InvocationList.IndexOfByPredicate([=](const UnicastDelegateType& Delegate) {
			return Delegate.GetFunctionName() == InFunctionName && Delegate.IsBoundToObjectEvenIfUnreachable(InObject);
		});

		if (FoundDelegate != INDEX_NONE)
		{
			InvocationList.RemoveAtSwap(FoundDelegate, EAllowShrinking::No);
		}
	}

	/** Cleans up any delegates in our invocation list that have expired (performance is O(N)) */
	void CompactInvocationList() const
	{
		InvocationList.RemoveAllSwap([](const UnicastDelegateType& Delegate){
			return Delegate.IsCompactable();
		});
	}

protected:

	/** Ordered list functions to invoke when the Broadcast function is called */
	mutable InvocationListType InvocationList;		// Mutable so that we can housekeep list even with 'const' broadcasts

	// Declare ourselves as a friend of FMulticastDelegateProperty so that it can access our function list
	friend class FMulticastDelegateProperty;
	friend class FMulticastInlineDelegateProperty;
	friend class FMulticastSparseDelegateProperty;

	// 
	friend class FCallDelegateHelper;

	friend struct TIsZeroConstructType<TMulticastScriptDelegate>;
};


template<typename ThreadSafetyMode> 
struct TIsZeroConstructType<TMulticastScriptDelegate<ThreadSafetyMode> >
{ 
	static constexpr bool Value = TIsZeroConstructType<typename TMulticastScriptDelegate<ThreadSafetyMode>::InvocationListType>::Value;
};
