// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes made in Dev-Sequencer stream
struct FSequencerObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// Per-platform overrides player overrides for media sources changed name and type.
		RenameMediaSourcePlatformPlayers,

		// Enable root motion isn't the right flag to use, but force root lock
		ConvertEnableRootMotionToForceRootLock,

		// Convert multiple rows to tracks
		ConvertMultipleRowsToTracks,

		// When finished now defaults to restore state
		WhenFinishedDefaultsToRestoreState,

		// EvaluationTree added
		EvaluationTree,

		// When finished now defaults to project default
		WhenFinishedDefaultsToProjectDefault,

		// When finished now defaults to project default
		FloatToIntConversion,

		// Purged old spawnable blueprint classes from level sequence assets
		PurgeSpawnableBlueprints,

		// Finish UMG evaluation on end
		FinishUMGEvaluation,

		// Manual serialization of float channel
		SerializeFloatChannel,

		// Change the linear keys so they act the old way and interpolate always.
		ModifyLinearKeysForOldInterp,

		// Full Manual serialization of float channel
		SerializeFloatChannelCompletely,

		// Set ContinuouslyRespawn to false by default, added FMovieSceneSpawnable::bNetAddressableName
		SpawnableImprovements,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

private:
	FSequencerObjectVersion() {}
};
