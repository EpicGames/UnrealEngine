// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/EnableIf.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/UniquePtr.h"
#include "UObject/StrongObjectPtrTemplatesFwd.h"

class UObject;

namespace UEStrongObjectPtr_Private
{
	struct FInternalReferenceCollectorReferencerNameProvider
	{
	};

	COREUOBJECT_API void ReleaseUObject(const UObject*);
}

/**
 * Take a ref-count on a UObject to prevent it from being GC'd while this guard is in scope.
 */
template <typename ObjectType, typename ReferencerNameProvider>
class TStrongObjectPtr
{
public:
	using ElementType = ObjectType;

	[[nodiscard]] TStrongObjectPtr(TStrongObjectPtr&& InOther)
		: Object(InOther.Object)
	{
		InOther.Object = nullptr;
	}

	TStrongObjectPtr& operator=(TStrongObjectPtr&& InOther)
	{
		if (this != &InOther)
		{
			Reset();
			Object = InOther.Object;
			InOther.Object = nullptr;
		}
		return *this;
	}

	UE_FORCEINLINE_HINT ~TStrongObjectPtr()
	{
		Reset();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT TStrongObjectPtr(TYPE_OF_NULLPTR = nullptr)
	{
		static_assert(TPointerIsConvertibleFromTo<ObjectType, const volatile UObject>::Value, "TStrongObjectPtr can only be constructed with UObject types");
	}

	[[nodiscard]] inline explicit TStrongObjectPtr(ObjectType* InObject)
	{
		static_assert(TPointerIsConvertibleFromTo<ObjectType, const volatile UObject>::Value, "TStrongObjectPtr can only be constructed with UObject types");
		Reset(InObject);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT TStrongObjectPtr(const TStrongObjectPtr& InOther)
	{
		Reset(InOther.Get());
	}

	template <
		typename OtherObjectType,
		typename OtherReferencerNameProvider
		UE_REQUIRES(std::is_convertible_v<OtherObjectType*, ObjectType*>)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT TStrongObjectPtr(const TStrongObjectPtr<OtherObjectType, OtherReferencerNameProvider>& InOther)
	{
		Reset(InOther.Get());
	}

	inline TStrongObjectPtr& operator=(const TStrongObjectPtr& InOther)
	{
		Reset(InOther.Get());
		return *this;
	}

	template <
		typename OtherObjectType,
		typename OtherReferencerNameProvider
		UE_REQUIRES(std::is_convertible_v<OtherObjectType*, ObjectType*>)
	>
	inline TStrongObjectPtr& operator=(const TStrongObjectPtr<OtherObjectType, OtherReferencerNameProvider>& InOther)
	{
		Reset(InOther.Get());
		return *this;
	}

	[[nodiscard]] inline ObjectType& operator*() const
	{
		check(IsValid());
		return *(ObjectType*)Get();
	}

	[[nodiscard]] inline ObjectType* operator->() const
	{
		check(IsValid());
		return (ObjectType*)Get();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValid() const
	{
		return Object != nullptr;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
	{
		return IsValid();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT ObjectType* Get() const
	{
		return (ObjectType*)Object;
	}

	inline void Reset()
	{
		if (Object)
		{
			// UObject type is forward declared, ReleaseRef() is not known.
			// So move the implementation to the cpp file instead.
			UEStrongObjectPtr_Private::ReleaseUObject(Object);
			Object = nullptr;
		}
	}
	
private:
	template<class T, class TWeakObjectPtrBase> friend struct TWeakObjectPtr;

	// Attach an object without incrementing its ref-count.
	inline void Attach(ObjectType* InNewObject)
	{
		Reset();
		Object = InNewObject;
	}

	// Detach the current owned object without decrementing its ref-count.
	inline ObjectType* Detach()
	{
		ObjectType* DetachedObject = Get();
		Object = nullptr;
		return DetachedObject;
	}

public:
	inline void Reset(ObjectType* InNewObject)
	{
		if (InNewObject)
		{
			if (Object == InNewObject)
			{
				return;
			}

			if (Object)
			{
				// UObject type is forward declared, ReleaseRef() is not known.
				// So move the implementation to the cpp file instead.
				UEStrongObjectPtr_Private::ReleaseUObject(Object);
			}
			InNewObject->AddRef();
			Object = InNewObject;
		}
		else
		{
			Reset();
		}
	}

	[[nodiscard]] UE_FORCEINLINE_HINT friend uint32 GetTypeHash(const TStrongObjectPtr& InStrongObjectPtr)
	{
		return GetTypeHash(InStrongObjectPtr.Get());
	}

private:
	// Store as UObject to allow forward declarations without having to fully resolve ObjectType before construction.
	// This is required because the destructor calls Reset, which need to be fully resolved at declaration.
	const UObject* Object{ nullptr };

	[[nodiscard]] friend UE_FORCEINLINE_HINT bool operator==(const TStrongObjectPtr& InLHS, const TStrongObjectPtr& InRHS)
	{
		return InLHS.Get() == InRHS.Get();
	}

	[[nodiscard]] friend UE_FORCEINLINE_HINT bool operator==(const TStrongObjectPtr& InLHS, TYPE_OF_NULLPTR)
	{
		return !InLHS.IsValid();
	}

	[[nodiscard]] friend UE_FORCEINLINE_HINT bool operator==(TYPE_OF_NULLPTR, const TStrongObjectPtr& InRHS)
	{
		return !InRHS.IsValid();
	}

	template <
		typename RHSObjectType,
		typename RHSReferencerNameProvider
		UE_REQUIRES(UE_REQUIRES_EXPR(std::declval<ObjectType*>() == std::declval<RHSObjectType*>()))
	>
	[[nodiscard]] friend UE_FORCEINLINE_HINT bool operator==(const TStrongObjectPtr& InLHS, const TStrongObjectPtr<RHSObjectType, RHSReferencerNameProvider>& InRHS)
	{
		return InLHS.Get() == InRHS.Get();
	}

	template <
		typename LHSObjectType,
		typename LHSReferencerNameProvider
		UE_REQUIRES(UE_REQUIRES_EXPR(std::declval<LHSObjectType*>() == std::declval<ObjectType*>()))
	>
	[[nodiscard]] friend UE_FORCEINLINE_HINT bool operator==(const TStrongObjectPtr<LHSObjectType, LHSReferencerNameProvider>& InLHS, const TStrongObjectPtr& InRHS)
	{
		return InLHS.Get() == InRHS.Get();
	}

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	[[nodiscard]] friend UE_FORCEINLINE_HINT bool operator!=(const TStrongObjectPtr& InLHS, const TStrongObjectPtr& InRHS)
	{
		return InLHS.Get() != InRHS.Get();
	}

	[[nodiscard]] friend UE_FORCEINLINE_HINT bool operator!=(const TStrongObjectPtr& InLHS, TYPE_OF_NULLPTR)
	{
		return InLHS.IsValid();
	}

	[[nodiscard]] friend UE_FORCEINLINE_HINT bool operator!=(TYPE_OF_NULLPTR, const TStrongObjectPtr& InRHS)
	{
		return InRHS.IsValid();
	}

	template <
		typename RHSObjectType,
		typename RHSReferencerNameProvider
		UE_REQUIRES(UE_REQUIRES_EXPR(std::declval<ObjectType*>() == std::declval<RHSObjectType*>()))
	>
	[[nodiscard]] friend UE_FORCEINLINE_HINT bool operator!=(const TStrongObjectPtr& InLHS, const TStrongObjectPtr<RHSObjectType, RHSReferencerNameProvider>& InRHS)
	{
		return InLHS.Get() != InRHS.Get();
	}

	template <
		typename LHSObjectType,
		typename LHSReferencerNameProvider
		UE_REQUIRES(UE_REQUIRES_EXPR(std::declval<LHSObjectType*>() == std::declval<ObjectType*>()))
	>
	[[nodiscard]] friend UE_FORCEINLINE_HINT bool operator!=(const TStrongObjectPtr<LHSObjectType, LHSReferencerNameProvider>& InLHS, const TStrongObjectPtr& InRHS)
	{
		return InLHS.Get() != InRHS.Get();
	}
#endif
};
