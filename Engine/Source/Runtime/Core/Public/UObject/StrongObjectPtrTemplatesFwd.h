// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UObject;

namespace UEStrongObjectPtr_Private
{
	struct FInternalReferenceCollectorReferencerNameProvider;
}

/**
 * TStrongObjectPtr is a strong pointer to a UObject.
 * It can return nullptr if it has not been initialized or has been constructed from a weak ptr that is already garbage collected.
 * It prevents an object from being garbage collected.
 * It can't be directly used across a network.
 *
 * Most often it is used when you explicitly want to prevent something from being garbage collected.
 */
template <typename ObjectType, typename ReferencerNameProvider = UEStrongObjectPtr_Private::FInternalReferenceCollectorReferencerNameProvider>
class TStrongObjectPtr;
