// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(DEFINE_UE5_MAIN_VERSIONS)
#define UE5_MAIN_VERSION(Version, ID) Version,
#elif defined(CHECK_UE5_MAIN_VERSIONS)
#define UE5_MAIN_VERSION(Version, ID) static_assert(FUE5MainStreamObjectVersion::Version == ID);
#endif

// Before any version changes were made
UE5_MAIN_VERSION(BeforeCustomVersionWasAdded, 0)

// Nanite data added to Chaos geometry collections
UE5_MAIN_VERSION(GeometryCollectionNaniteData, 1)

// Nanite Geometry Collection data moved to DDC
UE5_MAIN_VERSION(GeometryCollectionNaniteDDC, 2)
		
// Removing SourceAnimationData, animation layering is now applied during compression
UE5_MAIN_VERSION(RemovingSourceAnimationData, 3)

// New MeshDescription format.
// This is the correct versioning for MeshDescription changes which were added to ReleaseObjectVersion.
UE5_MAIN_VERSION(MeshDescriptionNewFormat, 4)

// Serialize GridGuid in PartitionActorDesc
UE5_MAIN_VERSION(PartitionActorDescSerializeGridGuid, 5)

// Set PKG_ContainsMapData on external actor packages
UE5_MAIN_VERSION(ExternalActorsMapDataPackageFlag, 6)

// Added a new configurable BlendProfileMode that the user can setup to control the behavior of blend profiles.
UE5_MAIN_VERSION(AnimationAddedBlendProfileModes, 7)

// Serialize DataLayers in WorldPartitionActorDesc
UE5_MAIN_VERSION(WorldPartitionActorDescSerializeDataLayers, 8)

// Renaming UAnimSequence::NumFrames to NumberOfKeys, as that what is actually contains.
UE5_MAIN_VERSION(RenamingAnimationNumFrames, 9)

// Serialize HLODLayer in WorldPartition HLODActorDesc
UE5_MAIN_VERSION(WorldPartitionHLODActorDescSerializeHLODLayer, 10)

// Fixed Nanite Geometry Collection cooked data
UE5_MAIN_VERSION(GeometryCollectionNaniteCooked, 11)
			
// Added bCooked to UFontFace assets
UE5_MAIN_VERSION(AddedCookedBoolFontFaceAssets, 12)

// Serialize CellHash in WorldPartition HLODActorDesc
UE5_MAIN_VERSION(WorldPartitionHLODActorDescSerializeCellHash, 13)

// Nanite data is now transient in Geometry Collection similar to how RenderData is transient in StaticMesh.
UE5_MAIN_VERSION(GeometryCollectionNaniteTransient, 14)

// Added FLandscapeSplineActorDesc
UE5_MAIN_VERSION(AddedLandscapeSplineActorDesc, 15)

// Added support for per-object collision constraint flag. [Chaos]
UE5_MAIN_VERSION(AddCollisionConstraintFlag, 16)

// Initial Mantle Serialize Version
UE5_MAIN_VERSION(MantleDbSerialize, 17)

// Animation sync groups explicitly specify sync method
UE5_MAIN_VERSION(AnimSyncGroupsExplicitSyncMethod, 18)

// Fixup FLandscapeActorDesc Grid indices
UE5_MAIN_VERSION(FLandscapeActorDescFixupGridIndices, 19)

// FoliageType with HLOD support
UE5_MAIN_VERSION(FoliageTypeIncludeInHLOD, 20)

// Introducing UAnimDataModel sub-object for UAnimSequenceBase containing all animation source data
UE5_MAIN_VERSION(IntroducingAnimationDataModel, 21)

// Serialize ActorLabel in WorldPartitionActorDesc
UE5_MAIN_VERSION(WorldPartitionActorDescSerializeActorLabel, 22)

// Fix WorldPartitionActorDesc serialization archive not persistent
UE5_MAIN_VERSION(WorldPartitionActorDescSerializeArchivePersistent, 23)

// Fix potentially duplicated actors when using ForceExternalActorLevelReference
UE5_MAIN_VERSION(FixForceExternalActorLevelReferenceDuplicates, 24)

// Make UMeshDescriptionBase serializable
UE5_MAIN_VERSION(SerializeMeshDescriptionBase, 25)

// Chaos FConvex uses array of FVec3s for vertices instead of particles
UE5_MAIN_VERSION(ConvexUsesVerticesArray, 26)

// Serialize HLOD info in WorldPartitionActorDesc
UE5_MAIN_VERSION(WorldPartitionActorDescSerializeHLODInfo, 27)

// Expose particle Disabled flag to the game thread
UE5_MAIN_VERSION(AddDisabledFlag, 28)

// Moving animation custom attributes from AnimationSequence to UAnimDataModel
UE5_MAIN_VERSION(MoveCustomAttributesToDataModel, 29)

// Use of triangulation at runtime in BlendSpace
UE5_MAIN_VERSION(BlendSpaceRuntimeTriangulation, 30)

// Fix to the Cubic smoothing, plus introduction of new smoothing types
UE5_MAIN_VERSION(BlendSpaceSmoothingImprovements, 31)

// Removing Tessellation parameters from Materials
UE5_MAIN_VERSION(RemovingTessellationParameters, 32)

// Sparse class data serializes its associated structure to allow for BP types to be used
UE5_MAIN_VERSION(SparseClassDataStructSerialization, 33)

// PackedLevelInstance bounds fix
UE5_MAIN_VERSION(PackedLevelInstanceBoundsFix, 34)

// Initial set of anim nodes converted to use constants held in sparse class data
UE5_MAIN_VERSION(AnimNodeConstantDataRefactorPhase0, 35)

// Explicitly serialized bSavedCachedExpressionData for Material(Instance)
UE5_MAIN_VERSION(MaterialSavedCachedData, 36)

// Remove explicit decal blend mode
UE5_MAIN_VERSION(RemoveDecalBlendMode, 37)

// Made directional lights be atmosphere lights by default
UE5_MAIN_VERSION(DirLightsAreAtmosphereLightsByDefault, 38)

// Changed how world partition streaming cells are named
UE5_MAIN_VERSION(WorldPartitionStreamingCellsNamingShortened, 39)

// Changed how actor descriptors compute their bounds
UE5_MAIN_VERSION(WorldPartitionActorDescGetStreamingBounds, 40)

// Switch FMeshDescriptionBulkData to use virtualized bulkdata
UE5_MAIN_VERSION(MeshDescriptionVirtualization, 41)
		
// Switch FTextureSource to use virtualized bulkdata
UE5_MAIN_VERSION(TextureSourceVirtualization, 42)

// RigVM to store more information alongside the Copy Operator
UE5_MAIN_VERSION(RigVMCopyOpStoreNumBytes, 43)

// Expanded separate translucency into multiple passes
UE5_MAIN_VERSION(MaterialTranslucencyPass, 44)

// Chaos FGeometryCollectionObject user defined collision shapes support
UE5_MAIN_VERSION(GeometryCollectionUserDefinedCollisionShapes, 45)

// Removed the AtmosphericFog component with conversion to SkyAtmosphere component
UE5_MAIN_VERSION(RemovedAtmosphericFog, 46)

// The SkyAtmosphere now light up the heightfog by default, and by default the height fog has a black color.
UE5_MAIN_VERSION(SkyAtmosphereAffectsHeightFogWithBetterDefault, 47)

// Ordering of samples in BlendSpace
UE5_MAIN_VERSION(BlendSpaceSampleOrdering, 48)

// No longer bake MassToLocal transform into recorded transform data in GeometryCollection caching
UE5_MAIN_VERSION(GeometryCollectionCacheRemovesMassToLocal, 49)

// UEdGraphPin serializes SourceIndex
UE5_MAIN_VERSION(EdGraphPinSourceIndex, 50)

// Change texture bulkdatas to have unique guids
UE5_MAIN_VERSION(VirtualizedBulkDataHaveUniqueGuids, 51)

// Introduce RigVM Memory Class Object
UE5_MAIN_VERSION(RigVMMemoryStorageObject, 52)

// Ray tracing shadows have three states now (Disabled, Use Project Settings, Enabled)
UE5_MAIN_VERSION(RayTracedShadowsType, 53)

// Add bVisibleInRayTracing flag to Skeletal Mesh Sections
UE5_MAIN_VERSION(SkelMeshSectionVisibleInRayTracingFlagAdded, 54)

// Add generic tagging of all anim graph nodes in anim blueprints
UE5_MAIN_VERSION(AnimGraphNodeTaggingAdded, 55)
		
// Add custom version to FDynamicMesh3
UE5_MAIN_VERSION(DynamicMeshCompactedSerialization, 56)

// Remove the inline reduction bulkdata and replace it by a simple vertex and triangle count cache
UE5_MAIN_VERSION(ConvertReductionBaseSkeletalMeshBulkDataToInlineReductionCacheData, 57)

// Added some new MeshInfo to the FSkeletalMeshLODModel class.
UE5_MAIN_VERSION(SkeletalMeshLODModelMeshInfo, 58)
		
// Add Texture DoScaleMipsForAlphaCoverage
UE5_MAIN_VERSION(TextureDoScaleMipsForAlphaCoverage, 59)

// Fixed default value of volumetric cloud to be exact match with main view, more expenssive but we let user choosing how to lower the quality.
UE5_MAIN_VERSION(VolumetricCloudReflectionSampleCountDefaultUpdate, 60)

// Use special BVH for TriangleMesh, instead of the AABBTree
UE5_MAIN_VERSION(UseTriangleMeshBVH, 61)

// FDynamicMeshAttributeSet has Weight Maps. TDynamicAttributeBase serializes its name.
UE5_MAIN_VERSION(DynamicMeshAttributesWeightMapsAndNames, 62)

// Switching FK control naming scheme to incorporate _CURVE for curve controls
UE5_MAIN_VERSION(FKControlNamingScheme, 63)

// Fix-up for FRichCurveKey::TangentWeightMode, which were found to contain invalid value w.r.t the enum-type
UE5_MAIN_VERSION(RichCurveKeyInvalidTangentMode, 64)

// Enforcing new automatic tangent behaviour, enforcing auto-tangents for Key0 and KeyN to be flat, for Animation Assets.
UE5_MAIN_VERSION(ForceUpdateAnimationAssetCurveTangents, 65)

// SoundWave Update to use EditorBuildData for it's RawData
UE5_MAIN_VERSION(SoundWaveVirtualizationUpdate, 66)

// Fix material feature level nodes to account for new SM6 input pin.
UE5_MAIN_VERSION(MaterialFeatureLevelNodeFixForSM6, 67)

// Fix material feature level nodes to account for new SM6 input pin.
UE5_MAIN_VERSION(GeometryCollectionPerChildDamageThreshold, 68)

// Move some Chaos flags into a bitfield
UE5_MAIN_VERSION(AddRigidParticleControlFlags, 69)

// Allow each LiveLink controller to specify its own component to control
UE5_MAIN_VERSION(LiveLinkComponentPickerPerController, 70)

// Remove Faces in Triangle Mesh BVH
UE5_MAIN_VERSION(RemoveTriangleMeshBVHFaces, 71)

// Moving all nodal offset handling to Lens Component
UE5_MAIN_VERSION(LensComponentNodalOffset, 72)

// GPU none interpolated spawning no longer calls the update script
UE5_MAIN_VERSION(FixGpuAlwaysRunningUpdateScriptNoneInterpolated, 73)

// World partition streaming policy serialization only for cooked builds
UE5_MAIN_VERSION(WorldPartitionSerializeStreamingPolicyOnCook, 74)

// Remove serialization of bounds relevant from  WorldPartitionActorDesc
UE5_MAIN_VERSION(WorldPartitionActorDescRemoveBoundsRelevantSerialization, 75)

// Added IAnimationDataModel interface and replace UObject based representation for Animation Assets
// This version had to be undone. Animation assets saved between this and the subsequent backout version
// will be unable to be loaded
UE5_MAIN_VERSION(AnimationDataModelInterface_BackedOut, 76)

// Deprecate LandscapeSplineActorDesc
UE5_MAIN_VERSION(LandscapeSplineActorDescDeprecation, 77)

// Revert the IAnimationDataModel changes. Animation assets 
UE5_MAIN_VERSION(BackoutAnimationDataModelInterface, 78)

// Made stationary local and skylights behave similar to SM5
UE5_MAIN_VERSION(MobileStationaryLocalLights, 79)
// Made ManagedArrayCollection::FValueType::Value always serialize when FValueType is
UE5_MAIN_VERSION(ManagedArrayCollectionAlwaysSerializeValue, 80)

// Moving all distortion handling to Lens Component
UE5_MAIN_VERSION(LensComponentDistortion, 81)

// Updated image media source path resolution logic
UE5_MAIN_VERSION(ImgMediaPathResolutionWithEngineOrProjectTokens, 82)

// Add low resolution data in Height Field
UE5_MAIN_VERSION(AddLowResolutionHeightField, 83)

// Low resolution data in Height Field will store one height for (6x6) 36 cells
UE5_MAIN_VERSION(DecreaseLowResolutionHeightField, 84)

// Add damage propagation settings to geometry collections
UE5_MAIN_VERSION(GeometryCollectionDamagePropagationData, 85)

// Wheel friction forces are now applied at tire contact point
UE5_MAIN_VERSION(VehicleFrictionForcePositionChange, 86)

// Add flag to override MeshDeformer on a SkinnedMeshComponent.
UE5_MAIN_VERSION(AddSetMeshDeformerFlag, 87)

// Replace FNames for class/actor paths with FSoftObjectPath
UE5_MAIN_VERSION(WorldPartitionActorDescActorAndClassPaths, 88)

// Reintroducing AnimationDataModelInterface_BackedOut changes
UE5_MAIN_VERSION(ReintroduceAnimationDataModelInterface, 89)

// Support 16-bit skin weights on SkeletalMesh
UE5_MAIN_VERSION(IncreasedSkinWeightPrecision, 90)

// bIsUsedWithVolumetricCloud flag auto conversion
UE5_MAIN_VERSION(MaterialHasIsUsedWithVolumetricCloudFlag, 91)

// bIsUsedWithVolumetricCloud flag auto conversion
UE5_MAIN_VERSION(UpdateHairDescriptionBulkData, 92)

// Added TransformScaleMethod pin to SpawnActorFromClass node
UE5_MAIN_VERSION(SpawnActorFromClassTransformScaleMethod, 93)

// Added support for the RigVM to run branches lazily
UE5_MAIN_VERSION(RigVMLazyEvaluation, 94)

// Adding additional object version to defer out-of-date pose asset warning until next resaves
UE5_MAIN_VERSION(PoseAssetRawDataGUIDUpdate, 95)

// Store function information (and compilation data) in blueprint generated class
UE5_MAIN_VERSION(RigVMSaveFunctionAccessInModel, 96)

// Store the RigVM execute context struct the VM uses in the archive
UE5_MAIN_VERSION(RigVMSerializeExecuteContextStruct, 97)

// Store the Visual Logger timestamp as a double
UE5_MAIN_VERSION(VisualLoggerTimeStampAsDouble, 98)

// Add ThinSurface instance override support
UE5_MAIN_VERSION(MaterialInstanceBasePropertyOverridesThinSurface, 99)

// Add refraction mode None, converted from legacy when the refraction pin is not plugged.
UE5_MAIN_VERSION(MaterialRefractionModeNone, 100)

// Store serialized graph function in the function data
UE5_MAIN_VERSION(RigVMSaveSerializedGraphInGraphFunctionData, 101)

// Animation Sequence now stores its frame-rate on a per-platform basis
UE5_MAIN_VERSION(PerPlatformAnimSequenceTargetFrameRate, 102)
		
// New default for number of attributes on 2d grids
UE5_MAIN_VERSION(NiagaraGrid2DDefaultUnnamedAttributesZero, 103)

// RigVM generated class refactor
UE5_MAIN_VERSION(RigVMGeneratedClass, 104)

// In certain cases, Blueprint pins with a PC_Object category would serialize a null PinSubCategoryObject
UE5_MAIN_VERSION(NullPinSubCategoryObjectFix, 105)

// Allow custom event nodes to use access specifiers
UE5_MAIN_VERSION(AccessSpecifiersForCustomEvents, 106)

// Explicit override of Groom's hair width
UE5_MAIN_VERSION(GroomAssetWidthOverride, 107)

// Smart names removed from animation systems
UE5_MAIN_VERSION(AnimationRemoveSmartNames, 108)

// Change the default for facing & alignment to be automatic
UE5_MAIN_VERSION(NiagaraSpriteRendererFacingAlignmentAutoDefault, 109)
		
// Change the default for facing & alignment to be automatic
UE5_MAIN_VERSION(GroomAssetRemoveInAssetSerialization, 110)

// Changed the material property connected bitmasks from 32bit to 64bit
UE5_MAIN_VERSION(IncreaseMaterialAttributesInputMask, 111)

// Combines proprties into a new binding so users can select constant or binding
UE5_MAIN_VERSION(NiagaraSimStageNumIterationsBindings, 112)

// Skeletal vertex attributes
UE5_MAIN_VERSION(SkeletalVertexAttributes, 113)
		
// Store the RigVM execute context struct the VM uses in the archive
UE5_MAIN_VERSION(RigVMExternalExecuteContextStruct, 114)

// serialization inputs and outputs as two different sections
UE5_MAIN_VERSION(DataflowSeparateInputOutputSerialization, 115)

// Cloth collection tether initialization
UE5_MAIN_VERSION(ClothCollectionTetherInitialization, 116)

// OpenColorIO transforms now serialize their generated texture(s) and shader code normally into the uasset.
UE5_MAIN_VERSION(OpenColorIOAssetCacheSerialization, 117)

// Cloth collection single lod schema
UE5_MAIN_VERSION(ClothCollectionSingleLodSchema, 118)

// Visual Logger format now includes a WorldTimeStamp in addition to TimeStamp for easier debugging between multiple instances.
UE5_MAIN_VERSION(VisualLoggerAddedSeparateWorldTime, 119)

// Added support for InstanceDataManagerSerialization and changed format for the instances to FTransform3f
UE5_MAIN_VERSION(SkinnedMeshInstanceDataSerializationV2, 120)
	
// Added new material compilation validation for runtime virtual textures
UE5_MAIN_VERSION(RuntimeVirtualTextureMaterialValidation, 121)

// Refactored the composite (plugin) camera actor references.
UE5_MAIN_VERSION(CompositeCameraReferenceRefactor, 122)

// Skeletal mesh section now contains a bool for whether it contains a unified bone map.
UE5_MAIN_VERSION(SkeletalMeshUnifiedBoneMap, 123)

// -----<new versions can be added above this line>-------------------------------------------------
#undef UE5_MAIN_VERSION