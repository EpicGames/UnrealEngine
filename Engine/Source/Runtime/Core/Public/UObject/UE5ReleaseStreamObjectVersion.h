// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "UObject/DevObjectVersion.h"

// Custom serialization version for changes made in //UE5/Release-* stream
struct FUE5ReleaseStreamObjectVersion
{
	enum Type
	{
		#define DEFINE_UE5_RELEASE_VERSIONS
		#include "UObject/UE5ReleaseStreamObjectVersions.inl"
		#undef DEFINE_UE5_RELEASE_VERSIONS
		
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

	static CORE_API TMap<FGuid, FGuid> GetSystemGuids();

	FUE5ReleaseStreamObjectVersion() = delete;
};

#define CHECK_UE5_RELEASE_VERSIONS
#include "UObject/UE5ReleaseStreamObjectVersions.inl"
#undef CHECK_UE5_RELEASE_VERSIONS
