// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(DEFINE_UE5_RELEASE_VERSIONS)
#define UE5_RELEASE_VERSION(Version, ID) Version,
#elif defined(CHECK_UE5_RELEASE_VERSIONS)
#define UE5_RELEASE_VERSION(Version, ID) static_assert(FUE5ReleaseStreamObjectVersion::Version == ID);
#endif

// Before any version changes were made
UE5_RELEASE_VERSION(BeforeCustomVersionWasAdded, 0)

// Added Lumen reflections to new reflection enum, changed defaults
UE5_RELEASE_VERSION(ReflectionMethodEnum, 1)

// Serialize HLOD info in WorldPartitionActorDesc
UE5_RELEASE_VERSION(WorldPartitionActorDescSerializeHLODInfo, 2)

// Removing Tessellation from materials and meshes.
UE5_RELEASE_VERSION(RemovingTessellation, 3)

// LevelInstance serialize runtime behavior
UE5_RELEASE_VERSION(LevelInstanceSerializeRuntimeBehavior, 4)   

// Refactoring Pose Asset runtime data structures
UE5_RELEASE_VERSION(PoseAssetRuntimeRefactor, 5)

// Serialize the folder path of actor descs
UE5_RELEASE_VERSION(WorldPartitionActorDescSerializeActorFolderPath, 6)

// Change hair strands vertex format
UE5_RELEASE_VERSION(HairStrandsVertexFormatChange, 7)

// Added max linear and angular speed to Chaos bodies
UE5_RELEASE_VERSION(AddChaosMaxLinearAngularSpeed, 8)

// PackedLevelInstance version
UE5_RELEASE_VERSION(PackedLevelInstanceVersion, 9)

// PackedLevelInstance bounds fix
UE5_RELEASE_VERSION(PackedLevelInstanceBoundsFix, 10)

// Custom property anim graph nodes (linked anim graphs, control rig etc.) now use optional pin manager
UE5_RELEASE_VERSION(CustomPropertyAnimGraphNodesUseOptionalPinManager, 11)

// Add native double and int64 support to FFormatArgumentData
UE5_RELEASE_VERSION(TextFormatArgumentData64bitSupport, 12)

// Material layer stacks are no longer considered 'static parameters'
UE5_RELEASE_VERSION(MaterialLayerStacksAreNotParameters, 13)

// CachedExpressionData is moved from UMaterial to UMaterialInterface
UE5_RELEASE_VERSION(MaterialInterfaceSavedCachedData, 14)

// Add support for multiple cloth deformer LODs to be able to raytrace cloth with a different LOD than the one it is rendered with
UE5_RELEASE_VERSION(AddClothMappingLODBias, 15)

// Add support for different external actor packaging schemes
UE5_RELEASE_VERSION(AddLevelActorPackagingScheme, 16)

// Add support for linking to the attached parent actor in WorldPartitionActorDesc
UE5_RELEASE_VERSION(WorldPartitionActorDescSerializeAttachParent, 17)

// Converted AActor GridPlacement to bIsSpatiallyLoaded flag
UE5_RELEASE_VERSION(ConvertedActorGridPlacementToSpatiallyLoadedFlag, 18)

// Fixup for bad default value for GridPlacement_DEPRECATED
UE5_RELEASE_VERSION(ActorGridPlacementDeprecateDefaultValueFixup, 19)

// PackedLevelActor started using FWorldPartitionActorDesc (not currently checked against but added as a security)
UE5_RELEASE_VERSION(PackedLevelActorUseWorldPartitionActorDesc, 20)

// Add support for actor folder objects
UE5_RELEASE_VERSION(AddLevelActorFolders, 21)

// Remove FSkeletalMeshLODModel bulk datas
UE5_RELEASE_VERSION(RemoveSkeletalMeshLODModelBulkDatas, 22)

// Exclude brightness from the EncodedHDRCubemap,
UE5_RELEASE_VERSION(ExcludeBrightnessFromEncodedHDRCubemap, 23)

// Unified volumetric cloud component quality sample count slider between main and reflection views for consistency
UE5_RELEASE_VERSION(VolumetricCloudSampleCountUnification, 24)

// Pose asset GUID generated from source AnimationSequence
UE5_RELEASE_VERSION(PoseAssetRawDataGUID, 25)

// Convolution bloom now take into account FPostProcessSettings::BloomIntensity for scatter dispersion.
UE5_RELEASE_VERSION(ConvolutionBloomIntensity, 26)

// Serialize FHLODSubActors instead of FGuids in WorldPartition HLODActorDesc
UE5_RELEASE_VERSION(WorldPartitionHLODActorDescSerializeHLODSubActors, 27)

// Large Worlds - serialize double types as doubles
UE5_RELEASE_VERSION(LargeWorldCoordinates, 28)

// Deserialize old BP float&double types as real numbers for pins
UE5_RELEASE_VERSION(BlueprintPinsUseRealNumbers, 29)

// Changed shadow defaults for directional light components, version needed to not affect old things
UE5_RELEASE_VERSION(UpdatedDirectionalLightShadowDefaults, 30)

// Refresh geometry collections that had not already generated convex bodies.
UE5_RELEASE_VERSION(GeometryCollectionConvexDefaults, 31)

// Add faster damping calculations to the cloth simulation and rename previous Damping parameter to LocalDamping.
UE5_RELEASE_VERSION(ChaosClothFasterDamping, 32)

// Serialize LandscapeActorGuid in FLandscapeActorDesc sub class.
UE5_RELEASE_VERSION(WorldPartitionLandscapeActorDescSerializeLandscapeActorGuid, 33)

// add inertia tensor and rotation of mass to convex
UE5_RELEASE_VERSION(AddedInertiaTensorAndRotationOfMassAddedToConvex, 34)

// Storing inertia tensor as vec3 instead of matrix.
UE5_RELEASE_VERSION(ChaosInertiaConvertedToVec3, 35)

// For Blueprint real numbers, ensure that legacy float data is serialized as single-precision
UE5_RELEASE_VERSION(SerializeFloatPinDefaultValuesAsSinglePrecision, 36)

// Upgrade the BlendMasks array in existing LayeredBoneBlend nodes
UE5_RELEASE_VERSION(AnimLayeredBoneBlendMasks, 37)

// Uses RG11B10 format to store the encoded reflection capture data on mobile
UE5_RELEASE_VERSION(StoreReflectionCaptureEncodedHDRDataInRG11B10Format, 38)

// Add WithSerializer type trait and implementation for FRawAnimSequenceTrack
UE5_RELEASE_VERSION(RawAnimSequenceTrackSerializer, 39)

// Removed font from FEditableTextBoxStyle, and added FTextBlockStyle instead.
UE5_RELEASE_VERSION(RemoveDuplicatedStyleInfo, 40)

// Added member reference to linked anim graphs
UE5_RELEASE_VERSION(LinkedAnimGraphMemberReference, 41)

// Changed default tangent behavior for new dynamic mesh components
UE5_RELEASE_VERSION(DynamicMeshComponentsDefaultUseExternalTangents, 42)

// Added resize methods to media capture
UE5_RELEASE_VERSION(MediaCaptureNewResizeMethods, 43)

// Function data stores a map from work to debug operands
UE5_RELEASE_VERSION(RigVMSaveDebugMapInGraphFunctionData, 44)

// Changed default Local Exposure Contrast Scale from 1.0 to 0.8
UE5_RELEASE_VERSION(LocalExposureDefaultChangeFrom1, 45)

// Serialize bActorIsListedInSceneOutliner in WorldPartitionActorDesc
UE5_RELEASE_VERSION(WorldPartitionActorDescSerializeActorIsListedInSceneOutliner, 46)

// Disabled opencolorio display configuration by default
UE5_RELEASE_VERSION(OpenColorIODisabledDisplayConfigurationDefault, 47)

// Serialize ExternalDataLayerAsset in WorldPartitionActorDesc
UE5_RELEASE_VERSION(WorldPartitionExternalDataLayers, 48)

// Fix Chaos Cloth fictitious angular scale bug that requires existing parameter rescaling.
UE5_RELEASE_VERSION(ChaosClothFictitiousAngularVelocitySubframeFix, 49)

// Store physics thread particles data in single precision
UE5_RELEASE_VERSION(SinglePrecisonParticleDataPT, 50)

//Orthographic Near and Far Plane Auto-resolve enabled by default
UE5_RELEASE_VERSION(OrthographicAutoNearFarPlane, 51)

// Fix a bug where BlendMask counts could get out of sync with BlendPose counts.
UE5_RELEASE_VERSION(AnimLayeredBoneBlendMasksFix, 52)

// Separated lens flare from bloom intensity
UE5_RELEASE_VERSION(BloomIndependentLensFlare, 53)

// Add settings to IAnimationDataModel GUID generation
UE5_RELEASE_VERSION(AnimModelGuidGenerationSettings, 54)

// Add support for Standalone HLOD
UE5_RELEASE_VERSION(WorldPartitionAddStandaloneHLODSupport, 55)

// Fixed the missing bounds for cloth assets that don't have them serialized
UE5_RELEASE_VERSION(RecalculateClothAssetSerializedBounds, 56)

// Composite plugin now uses its own derived scene capture components
UE5_RELEASE_VERSION(CompositePluginDerivedSceneCaptures, 57)

// Add option to output attributes on the PCG Duplicate Cross Section node on the Data domain
UE5_RELEASE_VERSION(ExtraOutputAttributesOnDataDomainPCG, 58)

// Media Profile: Changed storage of capture cameras list from lazy to soft pointers
UE5_RELEASE_VERSION(MediaProfilePluginCaptureCameraSoftPtr, 59)

// Reparameterize Spline in SplineComponent based on SplineCurves parameterization
UE5_RELEASE_VERSION(SplineComponentReparameterizeOnLoad, 60)

// Add solver and fabric property support to the schema based Cloth USD importer
UE5_RELEASE_VERSION(AddSimulationPropertySupportToClothUSDImportNodeV2, 61)

// Audio section: add bRepeating property alongside legacy bLooping
UE5_RELEASE_VERSION(AudioSectionRepeating, 62)

// mesh_part: Serialize body/shape associations separately, new Verse interface serialization, enforce unique shape names
UE5_RELEASE_VERSION(MeshPartAssetUserDataBodyShapeSeparateAndInterfaceFNameAndUniqueShapeNames, 63)

// USkeleton no longer serializes the FSmartNameContainer legacy payload; drained only for assets older than this version
UE5_RELEASE_VERSION(RemovedSmartNameContainerPayload, 64)

// Composite plugin: UCompositeDepthMeshComponent's default StaticMesh is no longer set on the CDO
UE5_RELEASE_VERSION(CompositeDepthMeshDefaultRemovedFromCDO, 65)

// Add RotationBlendSpace enum to FAnimNode_BlendProfileLayeredBlend, replacing bMeshSpaceRotationBlend
UE5_RELEASE_VERSION(BlendProfileLayeredBlendRotationSpace, 66)

// Add guarding against negative blend times in AlphaBlends
UE5_RELEASE_VERSION(AlphaBlendNegativeBlendTimeGuard, 67)

// Audio section: add bPlayUntilFinished property and stop using CompletionMode for play/stop decisions
UE5_RELEASE_VERSION(AudioSectionPlayUntilFinished, 68)

// -----<new versions can be added above this line>-------------------------------------------------
#undef UE5_RELEASE_VERSION