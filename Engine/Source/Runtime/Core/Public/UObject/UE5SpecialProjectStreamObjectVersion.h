// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes made in //UE5/Private-Frosty stream
struct FUE5SpecialProjectStreamObjectVersion
{
	enum Type
	{
		#define DEFINE_SPECIAL_PROJECT_VERSIONS
		#include "UObject/UE5SpecialProjectStreamObjectVersion.inl"
		#undef DEFINE_SPECIAL_PROJECT_VERSIONS

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

	static CORE_API TMap<FGuid, FGuid> GetSystemGuids();

	FUE5SpecialProjectStreamObjectVersion() = delete;
};

#define CHECK_SPECIAL_PROJECT_VERSIONS
#include "UObject/UE5SpecialProjectStreamObjectVersion.inl"
#undef CHECK_SPECIAL_PROJECT_VERSIONS
