// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(DEFINE_SPECIAL_PROJECT_VERSIONS)
#define UE5_SPECIAL_PROJECT_VERSION(Version, ID) Version,
#elif defined(CHECK_SPECIAL_PROJECT_VERSIONS)
#define UE5_SPECIAL_PROJECT_VERSION(Version, ID) static_assert(FUE5SpecialProjectStreamObjectVersion::Version == ID);
#endif

// Before any version changes were made
UE5_SPECIAL_PROJECT_VERSION(BeforeCustomVersionWasAdded, 0)

// Added HLODBatchingPolicy member to UPrimitiveComponent, which replaces the confusing bUseMaxLODAsImposter & bBatchImpostersAsInstances.
UE5_SPECIAL_PROJECT_VERSION(HLODBatchingPolicy, 1)

// Serialize scene components static bounds
UE5_SPECIAL_PROJECT_VERSION(SerializeSceneComponentStaticBounds, 2)

// Add the long range attachment tethers to the cloth asset to avoid a large hitch during the cloth's initialization.
UE5_SPECIAL_PROJECT_VERSION(ChaosClothAddTethersToCachedData, 3)

// Always serialize the actor label in cooked builds
UE5_SPECIAL_PROJECT_VERSION(SerializeActorLabelInCookedBuilds, 4)

// Changed world partition HLODs cells from FSotObjectPath to FName
UE5_SPECIAL_PROJECT_VERSION(ConvertWorldPartitionHLODsCellsToName, 5)

// Re-calculate the long range attachment to prevent kinematic tethers.
UE5_SPECIAL_PROJECT_VERSION(ChaosClothRemoveKinematicTethers, 6)

// Serializes the Morph Target render data for cooked platforms and the DDC
UE5_SPECIAL_PROJECT_VERSION(SerializeSkeletalMeshMorphTargetRenderData, 7)

// Strip the Morph Target source data for cooked builds
UE5_SPECIAL_PROJECT_VERSION(StripMorphTargetSourceDataForCookedBuilds, 8)

// StateTree now holds PropertyBag + GUID for root-level parameters rather than FStateTreeStateParameters. Access is protected by default and can be overriden through virtuals on UStateTreeEditorData derived classes.
UE5_SPECIAL_PROJECT_VERSION(StateTreeGlobalParameterChanges, 9)

// -----<new versions can be added above this line>-------------------------------------------------
#undef UE5_SPECIAL_PROJECT_VERSION