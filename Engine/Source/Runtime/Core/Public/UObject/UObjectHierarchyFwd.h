// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UObjectBase;
class UObjectBaseUtility;
class	UObject;
class		UField;
class			UEnum;
class			FProperty;
class				FObjectProperty;
class			UStruct;
class				UFunction;
class				UClass;
class				UScriptStruct;
class		FLinker;
class			FLinkerLoad;
class			FLinkerSave;
class		UPackage;
class		USystem;
class		UTextBuffer;
class		UPackageMap;
class		UObjectRedirector;
class FField;
class	FProperty;
class		FByteProperty;
class		FUInt16Property;
class		FUInt32Property;
class		FUInt64Property;
class		FInt8Property;
class		FInt16Property;
class		FIntProperty;
class		FInt64Property;
class		FBoolProperty;
class		FFloatProperty;
class		FDoubleProperty;
class		FObjectPropertyBase;
class		FObjectProperty;
class			FClassProperty;
class			FInterfaceProperty;
class			FWeakObjectProperty;
class			FLazyObjectProperty;
class			FSoftObjectProperty;
class				FSoftClassProperty;
class		FNameProperty;
class		FStructProperty;
class		FStrProperty;
class		FTextProperty;
class		FArrayProperty;
class		FDelegateProperty;
class		FMulticastDelegateProperty;
class			FMulticastInlineDelegateProperty;
class			FMulticastSparseDelegateProperty;
class		FMapProperty;
class		FSetProperty;
class		FEnumProperty;
class		FUtf8StrProperty;
class		FAnsiStrProperty;
class		FOptionalProperty;
class		FFieldPathProperty;
