// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

struct FNameEntryId;

//
// Macros.
//

// Define a message as an enumeration.
#define REGISTER_NAME(num,name) name = num,
enum class EName : uint32
{
	// Include all the hard-coded names
	#include "UnrealNames.inl"
	// Special constant for the last hard-coded name index
	MaxHardcodedNameIndex,
};
#undef REGISTER_NAME
// Define aliases for the old-style EName enum syntax
#define REGISTER_NAME(num,name) inline constexpr EName NAME_##name = EName::name;
#include "UnrealNames.inl"
#undef REGISTER_NAME

CORE_API const TCHAR* LexToString(EName Ename);

/** Index of highest hardcoded name to be replicated by index by the networking code
 * @warning: changing this number or making any change to the list of hardcoded names with index
 * less than this value breaks network compatibility, which by default checks for the same changelist
 * @note: names with a greater value than this can still be replicated, but they are sent as
 * strings instead of an efficient index
 *
 * @see ShouldReplicateENameAsInteger()
 */
#define MAX_NETWORKED_HARDCODED_NAME 410

CORE_API bool ShouldReplicateAsInteger(EName Ename, const class FName& Name);
