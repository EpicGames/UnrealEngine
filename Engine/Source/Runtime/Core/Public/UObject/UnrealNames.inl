// Copyright Epic Games, Inc. All Rights Reserved.

// [[ IncludeTool: Inline ]] // Markup to tell IncludeTool that this file is state changing and cannot be optimized out.

// Suppress any macros that may conflict with name constant symbols.
UE_PUSH_MACRO("TRUE")
#undef TRUE
UE_PUSH_MACRO("FALSE")
#undef FALSE

// Special zero value, meaning no name.
REGISTER_NAME(0,None)

// Class property types (name indices are significant for serialization).
REGISTER_NAME(1,ByteProperty)
REGISTER_NAME(2,IntProperty)
REGISTER_NAME(3,BoolProperty)
REGISTER_NAME(4,FloatProperty)
REGISTER_NAME(5,ObjectProperty) // ClassProperty shares the same tag
REGISTER_NAME(6,NameProperty)
REGISTER_NAME(7,DelegateProperty)
REGISTER_NAME(8,DoubleProperty)
REGISTER_NAME(9,ArrayProperty)
REGISTER_NAME(10,StructProperty)
REGISTER_NAME(11,VectorProperty)
REGISTER_NAME(12,RotatorProperty)
REGISTER_NAME(13,StrProperty)
REGISTER_NAME(14,TextProperty)
REGISTER_NAME(15,InterfaceProperty)
REGISTER_NAME(16,MulticastDelegateProperty)
//REGISTER_NAME(17,Available)
REGISTER_NAME(18,LazyObjectProperty)
REGISTER_NAME(19,SoftObjectProperty) // SoftClassProperty shares the same tag
REGISTER_NAME(20,Int64Property)
REGISTER_NAME(21,Int32Property)
REGISTER_NAME(22,Int16Property)
REGISTER_NAME(23,Int8Property)
REGISTER_NAME(24,UInt64Property)
REGISTER_NAME(25,UInt32Property)
REGISTER_NAME(26,UInt16Property)
//REGISTER_NAME(27,Available)
REGISTER_NAME(28,MapProperty)
REGISTER_NAME(29,SetProperty)

// Special packages.
REGISTER_NAME(30,Core)
REGISTER_NAME(31,Engine)
REGISTER_NAME(32,Editor)
REGISTER_NAME(33,CoreUObject)

// More class properties
REGISTER_NAME(34,EnumProperty)
REGISTER_NAME(35,OptionalProperty)
REGISTER_NAME(36,Utf8StrProperty)
REGISTER_NAME(37,AnsiStrProperty)
REGISTER_NAME(38,WeakObjectProperty)
REGISTER_NAME(39,MulticastSparseDelegateProperty)
REGISTER_NAME(40,FieldPathProperty)
REGISTER_NAME(41,MulticastInlineDelegateProperty)


// Special types.
REGISTER_NAME(50,Cylinder)
REGISTER_NAME(51,BoxSphereBounds)
REGISTER_NAME(52,Sphere)
REGISTER_NAME(53,Box)
REGISTER_NAME(54,Vector2D)
REGISTER_NAME(55,IntRect)
REGISTER_NAME(56,IntPoint)
REGISTER_NAME(57,Vector4)
REGISTER_NAME(58,Name)
REGISTER_NAME(59,Vector)
REGISTER_NAME(60,Rotator)
REGISTER_NAME(61,SHVector)
REGISTER_NAME(62,Color)
REGISTER_NAME(63,Plane)
REGISTER_NAME(64,Matrix)
REGISTER_NAME(65,LinearColor)
REGISTER_NAME(66,AdvanceFrame)
REGISTER_NAME(67,Pointer)
REGISTER_NAME(68,Double)
REGISTER_NAME(69,Quat)
REGISTER_NAME(70,Self)
REGISTER_NAME(71,Transform)
REGISTER_NAME(72,Vector3f)
REGISTER_NAME(73,Vector3d)
REGISTER_NAME(74,Plane4f)
REGISTER_NAME(75,Plane4d)
REGISTER_NAME(76,Matrix44f)
REGISTER_NAME(77,Matrix44d)
REGISTER_NAME(78,Quat4f)
REGISTER_NAME(79,Quat4d)
REGISTER_NAME(80,Transform3f)
REGISTER_NAME(81,Transform3d)
REGISTER_NAME(82,Box3f)
REGISTER_NAME(83,Box3d)
REGISTER_NAME(84,BoxSphereBounds3f)
REGISTER_NAME(85,BoxSphereBounds3d)
REGISTER_NAME(86,Vector4f)			
REGISTER_NAME(87,Vector4d)
REGISTER_NAME(88,Rotator3f)
REGISTER_NAME(89,Rotator3d)
REGISTER_NAME(90,Vector2f)
REGISTER_NAME(91,Vector2d)
REGISTER_NAME(92,Box2D)
REGISTER_NAME(93,Box2f)
REGISTER_NAME(94,Box2d)
REGISTER_NAME(95,IntVector)
REGISTER_NAME(96,IntVector4)
REGISTER_NAME(97,UintVector)
REGISTER_NAME(98,UintVector4)

// Object class names.
REGISTER_NAME(100,Object)
REGISTER_NAME(101,Camera)
REGISTER_NAME(102,Actor)
REGISTER_NAME(103,ObjectRedirector)
REGISTER_NAME(104,ObjectArchetype)
REGISTER_NAME(105,Class)
REGISTER_NAME(106,ScriptStruct)
REGISTER_NAME(107,Function)
REGISTER_NAME(108,Pawn)

REGISTER_NAME(109,VerseClass)
REGISTER_NAME(110,VerseStruct)
REGISTER_NAME(111,VerseEnum)

// Special types continued
REGISTER_NAME(150, Int32Vector)
REGISTER_NAME(151, Int64Vector)
REGISTER_NAME(152, Uint32Vector)
REGISTER_NAME(153, Uint64Vector)
REGISTER_NAME(154, Int32Vector4)
REGISTER_NAME(155, Int64Vector4)
REGISTER_NAME(156, Uint32Vector4)
REGISTER_NAME(157, Uint64Vector4)
REGISTER_NAME(158, IntVector2)
REGISTER_NAME(159, Int32Vector2)
REGISTER_NAME(160, Int64Vector2)
REGISTER_NAME(161, UintVector2)
REGISTER_NAME(162, Uint32Vector2)
REGISTER_NAME(163, Uint64Vector2)
REGISTER_NAME(164, UintPoint)
REGISTER_NAME(165, Int32Point)
REGISTER_NAME(166, Int64Point)
REGISTER_NAME(167, Uint32Point)
REGISTER_NAME(168, Uint64Point)
REGISTER_NAME(169, Ray)
REGISTER_NAME(170, Ray3f)
REGISTER_NAME(171, Ray3d)
REGISTER_NAME(172, Sphere3f)
REGISTER_NAME(173, Sphere3d)

// Misc.
REGISTER_NAME(200,State)
REGISTER_NAME(201,TRUE)
REGISTER_NAME(202,FALSE)
REGISTER_NAME(203,Enum)
REGISTER_NAME(204,Default)
REGISTER_NAME(205,Skip)
REGISTER_NAME(206,Input)
REGISTER_NAME(207,Package)
REGISTER_NAME(208,Groups)
REGISTER_NAME(209,Interface)
REGISTER_NAME(210,Components)
REGISTER_NAME(211,Global)
REGISTER_NAME(212,Super)
REGISTER_NAME(213,Outer)
REGISTER_NAME(214,Map)
REGISTER_NAME(215,Role)
REGISTER_NAME(216,RemoteRole)
REGISTER_NAME(217,PersistentLevel)
REGISTER_NAME(218,TheWorld)
REGISTER_NAME(219,PackageMetaData)
REGISTER_NAME(220,InitialState)
REGISTER_NAME(221,Game)
REGISTER_NAME(222,SelectionColor)
REGISTER_NAME(223,UI)
REGISTER_NAME(224,ExecuteUbergraph)
REGISTER_NAME(225,DeviceID)
REGISTER_NAME(226,RootStat)
REGISTER_NAME(227,MoveActor)
REGISTER_NAME(230,All) 
REGISTER_NAME(231,MeshEmitterVertexColor)
REGISTER_NAME(232,TextureOffsetParameter)
REGISTER_NAME(233,TextureScaleParameter)
REGISTER_NAME(234,ImpactVel)
REGISTER_NAME(235,SlideVel)
REGISTER_NAME(236,TextureOffset1Parameter)
REGISTER_NAME(237,MeshEmitterDynamicParameter)
REGISTER_NAME(238,ExpressionInput)
REGISTER_NAME(239,Untitled)
REGISTER_NAME(240,Timer)
REGISTER_NAME(241,Team)
REGISTER_NAME(242,Low)
REGISTER_NAME(243,High)
REGISTER_NAME(244,NetworkGUID)
REGISTER_NAME(245,GameThread)
REGISTER_NAME(246,RenderThread)
REGISTER_NAME(247,OtherChildren)
REGISTER_NAME(248,Location)
REGISTER_NAME(249,Rotation)
REGISTER_NAME(250,BSP)
REGISTER_NAME(251,EditorSettings)
REGISTER_NAME(252,AudioThread)
REGISTER_NAME(253,ID)
REGISTER_NAME(254,UserDefinedEnum)
REGISTER_NAME(255,Control)
REGISTER_NAME(256,Voice)
REGISTER_NAME(257, Zlib)
REGISTER_NAME(258, Gzip)
REGISTER_NAME(259, LZ4)
REGISTER_NAME(260, Mobile)
REGISTER_NAME(261, Oodle)
REGISTER_NAME(262, TrashedPackage)
REGISTER_NAME(263, MetaData)
REGISTER_NAME(264, AutoRTFMUnusedPackage)

// Online
REGISTER_NAME(280,DGram)
REGISTER_NAME(281,Stream)
REGISTER_NAME(282,GameNetDriver)
REGISTER_NAME(283,PendingNetDriver)
REGISTER_NAME(284,BeaconNetDriver)
REGISTER_NAME(285,FlushNetDormancy)
REGISTER_NAME(286,DemoNetDriver)
REGISTER_NAME(287,GameSession)
REGISTER_NAME(288,PartySession)
REGISTER_NAME(289,GamePort)
REGISTER_NAME(290,BeaconPort)
REGISTER_NAME(291,MeshPort)
REGISTER_NAME(292,MeshNetDriver)
REGISTER_NAME(293,LiveStreamVoice)
REGISTER_NAME(294,LiveStreamAnimation)
REGISTER_NAME(295,DataStream)

// Texture settings.
REGISTER_NAME(300,Linear)
REGISTER_NAME(301,Point)
REGISTER_NAME(302,Aniso)
REGISTER_NAME(303,LightMapResolution)

// Sound.
//REGISTER_NAME(310,)
REGISTER_NAME(311,UnGrouped)
REGISTER_NAME(312,VoiceChat)

// Optimized replication.
REGISTER_NAME(320,Playing)
REGISTER_NAME(322,Spectating)
REGISTER_NAME(325,Inactive)

// Log messages.
REGISTER_NAME(350,PerfWarning)
REGISTER_NAME(351,Info)
REGISTER_NAME(352,Init)
REGISTER_NAME(353,Exit)
REGISTER_NAME(354,Cmd)
REGISTER_NAME(355,Warning)
REGISTER_NAME(356,Error)

// File format backwards-compatibility.
REGISTER_NAME(400,FontCharacter)
REGISTER_NAME(401,InitChild2StartBone)
REGISTER_NAME(402,SoundCueLocalized)
REGISTER_NAME(403,SoundCue)
REGISTER_NAME(404,RawDistributionFloat)
REGISTER_NAME(405,RawDistributionVector)
REGISTER_NAME(406,InterpCurveFloat)
REGISTER_NAME(407,InterpCurveVector2D)
REGISTER_NAME(408,InterpCurveVector)
REGISTER_NAME(409,InterpCurveTwoVectors)
REGISTER_NAME(410,InterpCurveQuat)


REGISTER_NAME(411,FrameRate)

REGISTER_NAME(450,AI)
REGISTER_NAME(451,NavMesh)

REGISTER_NAME(500,PerformanceCapture)

// Special config names - not required to be consistent for network replication
REGISTER_NAME(600,EditorLayout)
REGISTER_NAME(601,EditorKeyBindings)
REGISTER_NAME(602,GameUserSettings)

// Commonly multicased names - defined globally to prevent non-determinism in package name tables
REGISTER_NAME(700,Filename)
REGISTER_NAME(701,Lerp)
REGISTER_NAME(702,Root)

// Restore any macros suppressed at the top of this file.
UE_POP_MACRO("TRUE")
UE_POP_MACRO("FALSE")
