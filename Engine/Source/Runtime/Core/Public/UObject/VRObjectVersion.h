// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes made in Dev-VR stream
struct FVRObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// Change UMotionControllerComponent from using EControllerHand to FName for motion source
		UseFNameInsteadOfEControllerHandForMotionSource,

		// Change how ARSessionConfig stores plane detection configuration from bitmask to bools
		UseBoolsForARSessionConfigPlaneDetectionConfiguration,

		// Change how UStereoLayerComponent stores additional properties for non-quad layer types
		UseSubobjectForStereoLayerShapeProperties,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	CORE_API const static FGuid GUID;

private:
	FVRObjectVersion() {}
};
