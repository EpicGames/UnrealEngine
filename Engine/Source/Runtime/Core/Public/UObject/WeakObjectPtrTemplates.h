// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/LosesQualifiersFromTo.h"
#include "Templates/Requires.h"
#include "Containers/Map.h"
#include "Misc/UEOps.h"
#include "UObject/WeakObjectPtrTemplatesFwd.h"
#include "UObject/StrongObjectPtrTemplatesFwd.h"
#include "UObject/RemoteObjectCore.h"

#include <type_traits>

struct FFieldPath;
struct FWeakObjectPtr;
struct FUObjectItem;
struct FRemoteObjectId;

/**
 * TWeakObjectPtr is the templated version of the generic FWeakObjectPtr
 */
template<class T, class TWeakObjectPtrBase>
struct TWeakObjectPtr
{
	friend struct FFieldPath;

	template <class, class>
	friend struct TWeakObjectPtr;

	friend struct FFieldPath;

	// Although templated, these parameters are not intended to be anything other than the default,
	// and are only templates for module organization reasons.
	static_assert(std::is_same_v<TWeakObjectPtrBase*, FWeakObjectPtr*>, "TWeakObjectPtrBase should not be overridden");

public:
	using ElementType = T;
	
	[[nodiscard]] TWeakObjectPtr() = default;
	[[nodiscard]] TWeakObjectPtr(const TWeakObjectPtr&) = default;
	TWeakObjectPtr& operator=(const TWeakObjectPtr&) = default;
	~TWeakObjectPtr() = default;

	/**
	 * Construct from a null pointer
	 */
	[[nodiscard]] FORCEINLINE TWeakObjectPtr(TYPE_OF_NULLPTR) :
		WeakPtr((UObject*)nullptr)
	{
	}

	/**
	 * Construct from an object pointer
	 * @param Object object to create a weak pointer to
	 */
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U, T*>)
	>
	[[nodiscard]] FORCEINLINE TWeakObjectPtr(U Object) :
		WeakPtr((const UObject*)Object)
	{
		// This static assert is in here rather than in the body of the class because we want
		// to be able to define TWeakObjectPtr<UUndefinedClass>.
		static_assert(std::is_convertible_v<T*, const volatile UObject*>, "TWeakObjectPtr can only be constructed with UObject types");
	}

	/**
	 * Construct from another weak pointer of another type, intended for derived-to-base conversions
	 * @param Other weak pointer to copy from
	 */
	template <
		typename OtherT
		UE_REQUIRES(std::is_convertible_v<OtherT*, T*>)
	>
	[[nodiscard]] FORCEINLINE TWeakObjectPtr(const TWeakObjectPtr<OtherT, TWeakObjectPtrBase>& Other) :
		WeakPtr(Other.WeakPtr) // we do a C-style cast to private base here to avoid clang 3.6.0 compilation problems with friend declarations
	{
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	explicit TWeakObjectPtr(const FRemoteObjectId& RemoteId)
		: WeakPtr(RemoteId)
	{
	}
#endif

	/**
	 * Reset the weak pointer back to the null state
	 */
	FORCEINLINE void Reset()
	{
		WeakPtr.Reset();
	}

	/**  
	 * Copy from an object pointer
	 * @param Object object to create a weak pointer to
	 */
	template <
		typename U
		UE_REQUIRES(!TLosesQualifiersFromTo_V<U, T>)
	>
	FORCEINLINE TWeakObjectPtr& operator=(U* Object)
	{
		T* TempObject = Object;
		WeakPtr = TempObject;
		return *this;
	}

	/**  
	 * Assign from another weak pointer, intended for derived-to-base conversions
	 * @param Other weak pointer to copy from
	 */
	template <
		typename OtherT
		UE_REQUIRES(std::is_convertible_v<OtherT*, T*>)
	>
	FORCEINLINE TWeakObjectPtr& operator=(const TWeakObjectPtr<OtherT, TWeakObjectPtrBase>& Other)
	{
		WeakPtr = Other.WeakPtr;

		return *this;
	}

	/**  
	 * Dereference the weak pointer
	 * @param bEvenIfPendingKill if this is true, pendingkill objects are considered valid
	 * @return nullptr if this object is gone or the weak pointer is explicitly null, otherwise a valid uobject pointer
	 */
	[[nodiscard]] FORCEINLINE T* Get(bool bEvenIfPendingKill) const
	{
		return (T*)WeakPtr.Get(bEvenIfPendingKill);
	}

	/**  
	 * Dereference the weak pointer. This is an optimized version implying bEvenIfPendingKill=false.
	 */
	[[nodiscard]] FORCEINLINE T* Get(/*bool bEvenIfPendingKill = false*/) const
	{
		return (T*)WeakPtr.Get();
	}

	/**
	 * Pin the weak pointer and get a strongptr.
	 * @param bEvenIfPendingKill if this is true, pendingkill objects are considered valid
	 * @return nullptr if this object is gone or the weak pointer is explicitly null, otherwise a valid uobject pointer
	 */
	[[nodiscard]] FORCEINLINE TStrongObjectPtr<T> Pin(bool bEvenIfPendingKill) const
	{
		TStrongObjectPtr<T> StrongPtr;
		StrongPtr.Attach((T*)WeakPtr.Pin(bEvenIfPendingKill).Detach());
		return StrongPtr;
	}

	/**
	 * Pin the weak pointer as a strong ptr. This is an optimized version implying bEvenIfPendingKill=false.
	 */
	[[nodiscard]] FORCEINLINE TStrongObjectPtr<T> Pin(/*bool bEvenIfPendingKill = false*/) const
	{
		TStrongObjectPtr<T> StrongPtr;
		StrongPtr.Attach((T*)WeakPtr.Pin().Detach());
		return StrongPtr;
	}

	/**
	* Pin the weak pointer and get a strongptr.
	* @param bOutPinValid true if garbage collection was not in progress, and OutResult was successfully captured, false if garbage collection was in progress and OutResult was not captured
	* @param bEvenIfPendingKill if this is true, pendingkill objects are considered valid
	* @return nullptr if this object is gone or the weak pointer is explicitly null, otherwise a valid uobject pointer
	*/
	[[nodiscard]] FORCEINLINE TStrongObjectPtr<T> TryPin(bool& bOutPinValid, bool bEvenIfPendingKill) const
	{
		TStrongObjectPtr<T> StrongPtr;
		StrongPtr.Attach((T*)WeakPtr.TryPin(bOutPinValid, bEvenIfPendingKill).Detach());
		return StrongPtr;
	}

	/**
	 * Pin the weak pointer as a strong ptr. This is an optimized version implying bEvenIfPendingKill=false.
	 */
	[[nodiscard]] FORCEINLINE TStrongObjectPtr<T> TryPin(bool& bOutPinValid /*, bool bEvenIfPendingKill = false */ ) const
	{
		return TryPin(bOutPinValid, false);
	}

	/** Deferences the weak pointer even if its marked RF_Unreachable. This is needed to resolve weak pointers during GC (such as ::AddReferenceObjects) */
	[[nodiscard]] FORCEINLINE T* GetEvenIfUnreachable() const
	{
		return (T*)WeakPtr.GetEvenIfUnreachable();
	}

	/**  
	 * Dereference the weak pointer
	 */
	[[nodiscard]] FORCEINLINE T& operator*() const
	{
		return *Get();
	}

	/**  
	 * Dereference the weak pointer
	 */
	[[nodiscard]] FORCEINLINE T* operator->() const
	{
		return Get();
	}

	// This is explicitly not added to avoid resolving weak pointers too often - use Get() once in a function.
	explicit operator bool() const = delete;

	/**
	 * Cast to the underlying generic FWeakObjectPtr type
	 */
	explicit operator TWeakObjectPtrBase() const
	{
		return WeakPtr;
	}

	/**  
	 * Test if this points to a live UObject.
	 * This should be done only when needed as excess resolution of the underlying pointer can cause performance issues.
	 * 
	 * @param bEvenIfPendingKill if this is true, pendingkill objects are considered valid
	 * @param bThreadsafeTest if true then function will just give you information whether referenced
	 *							UObject is gone forever (return false) or if it is still there (return true, no object flags checked).
	 *							This is required as without it IsValid can return false during the mark phase of the GC
	 *							due to the presence of the Unreachable flag.
	 * @return true if Get() would return a valid non-null pointer
	 */
	[[nodiscard]] FORCEINLINE bool IsValid(bool bEvenIfPendingKill, bool bThreadsafeTest = false) const
	{
		return WeakPtr.IsValid(bEvenIfPendingKill, bThreadsafeTest);
	}

	/**
	 * Test if this points to a live UObject. This is an optimized version implying bEvenIfPendingKill=false, bThreadsafeTest=false.
	 * This should be done only when needed as excess resolution of the underlying pointer can cause performance issues.
	 * Note that IsValid can not be used on another thread as it will incorrectly return false during the mark phase of the GC
	 * due to the Unreachable flag being set. (see bThreadsafeTest above)
	 * @return true if Get() would return a valid non-null pointer
	 */
	[[nodiscard]] FORCEINLINE bool IsValid(/*bool bEvenIfPendingKill = false, bool bThreadsafeTest = false*/) const
	{
		return WeakPtr.IsValid();
	}

	/**  
	 * Slightly different than !IsValid(), returns true if this used to point to a UObject, but doesn't any more and has not been assigned or reset in the mean time.
	 * @param bIncludingIfPendingKill if this is true, pendingkill objects are considered stale
	 * @param bThreadsafeTest set it to true when testing outside of Game Thread. Results in false if WeakObjPtr point to an existing object (no flags checked)
	 * @return true if this used to point at a real object but no longer does.
	 */
	[[nodiscard]] FORCEINLINE bool IsStale(bool bIncludingIfPendingKill = true, bool bThreadsafeTest = false) const
	{
		return WeakPtr.IsStale(bIncludingIfPendingKill, bThreadsafeTest);
	}
	
	/**
	 * Returns true if this pointer was explicitly assigned to null, was reset, or was never initialized.
	 * If this returns true, IsValid() and IsStale() will both return false.
	 */
	[[nodiscard]] FORCEINLINE bool IsExplicitlyNull() const
	{
		return WeakPtr.IsExplicitlyNull();
	}

	/**
	 * Returns true if two weak pointers were originally set to the same object, even if they are now stale
	 * @param Other weak pointer to compare to
	 */
	[[nodiscard]] FORCEINLINE bool HasSameIndexAndSerialNumber(const TWeakObjectPtr& Other) const
	{
		return WeakPtr.HasSameIndexAndSerialNumber(Other.WeakPtr);
	}

	/**
	 * Returns true if two weak pointers were originally set to the same object, even if they are now stale
	 * @param Other weak pointer to compare to
	 */
	template <
		typename OtherT
		UE_REQUIRES(UE_REQUIRES_EXPR((T*)nullptr == (OtherT*)nullptr))
	>
	[[nodiscard]] FORCEINLINE bool HasSameIndexAndSerialNumber(const TWeakObjectPtr<OtherT, TWeakObjectPtrBase>& Other) const
	{
		return WeakPtr.HasSameIndexAndSerialNumber(Other.WeakPtr);
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	FORCEINLINE bool HasSameObject(const UObject* Other) const
	{
		return WeakPtr.HasSameObject(Other);
	}

	FORCEINLINE auto GetRemoteId() const
	{
		return WeakPtr.GetRemoteId();
	}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

	UE_DEPRECATED(5.8, "Use GetResidence() instead.")
	[[nodiscard]] FORCEINLINE bool IsRemote() const
	{
		return WeakPtr.GetResidence() == EResidence::Remote;
	}

	[[nodiscard]] FORCEINLINE EResidence GetResidence() const
	{
		return WeakPtr.GetResidence();
	}

	/**
	 * Weak object pointer serialization, this forwards to FArchive::operator<<(struct FWeakObjectPtr&) or an override
	 */
	FORCEINLINE	void Serialize(FArchive& Ar)
	{
		Ar << WeakPtr;
	}

	/** Hash function. */
	[[nodiscard]] FORCEINLINE uint32 GetWeakPtrTypeHash() const
	{
		return WeakPtr.GetTypeHash();
	}

	/**
	 * Compare weak pointers for equality.
	 * If both pointers would return nullptr from Get() they count as equal even if they were not initialized to the same object.
	 * @param Other weak pointer to compare to
	 */
	template <typename RhsT, typename = decltype((T*)nullptr == (RhsT*)nullptr)>
	[[nodiscard]] FORCENOINLINE bool UEOpEquals(const TWeakObjectPtr<RhsT, TWeakObjectPtrBase>& Rhs) const
	{
		return WeakPtr == Rhs.WeakPtr;
	}

	template <typename RhsT, typename = decltype((T*)nullptr == (RhsT*)nullptr)>
	[[nodiscard]] FORCENOINLINE bool UEOpEquals(const RhsT* Rhs) const
	{
		// NOTE: this constructs a TWeakObjectPtrBase, which has some amount of overhead, so this may not be an efficient operation
		return WeakPtr == TWeakObjectPtrBase(Rhs);
	}

	[[nodiscard]] FORCENOINLINE bool UEOpEquals(TYPE_OF_NULLPTR) const
	{
		return !IsValid();
	}

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	/**
	 * Compare weak pointers for inequality
	 * @param Other weak pointer to compare to
	 */
	template <typename RhsT, typename = decltype((T*)nullptr != (RhsT*)nullptr)>
	[[nodiscard]] FORCEINLINE bool operator!=(const TWeakObjectPtr<RhsT, TWeakObjectPtrBase>& Rhs) const
	{
		return !(*this == Rhs);
	}

	template <typename RhsT, typename = decltype((T*)nullptr != (RhsT*)nullptr)>
	[[nodiscard]] FORCEINLINE bool operator!=(const RhsT* Rhs) const
	{
		return !(*this == Rhs);
	}

	[[nodiscard]] FORCEINLINE bool operator!=(TYPE_OF_NULLPTR) const
	{
		return !(*this == nullptr);
	}
#endif

private:
	FORCEINLINE FUObjectItem* Internal_GetObjectItem() const
	{
		return WeakPtr.Internal_GetObjectItem();
	}

	TWeakObjectPtrBase WeakPtr;
};

template <typename T>
TWeakObjectPtr(T*) -> TWeakObjectPtr<T>;

template <typename T>
TWeakObjectPtr(const TWeakObjectPtr<T>&) -> TWeakObjectPtr<T>;

// Helper function which deduces the type of the initializer
template <typename T>
[[nodiscard]] FORCEINLINE TWeakObjectPtr<T> MakeWeakObjectPtr(T* Ptr)
{
	return TWeakObjectPtr<T>(Ptr);
}


/**
 * SetKeyFuncs for TWeakObjectPtrs which allow the key to become stale without invalidating the set.
 */
template <typename ElementType, bool bInAllowDuplicateKeys = false>
struct TWeakObjectPtrSetKeyFuncs : DefaultKeyFuncs<ElementType, bInAllowDuplicateKeys>
{
	typedef typename DefaultKeyFuncs<ElementType, bInAllowDuplicateKeys>::KeyInitType KeyInitType;

	static FORCEINLINE bool Matches(KeyInitType A, KeyInitType B)
	{
		return A.HasSameIndexAndSerialNumber(B);
	}

	static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key);
	}
};

/**
 * MapKeyFuncs for TWeakObjectPtrs which allow the key to become stale without invalidating the map.
 */
template <typename KeyType, typename ValueType, bool bInAllowDuplicateKeys = false>
struct TWeakObjectPtrMapKeyFuncs : public TDefaultMapKeyFuncs<KeyType, ValueType, bInAllowDuplicateKeys>
{
	typedef typename TDefaultMapKeyFuncs<KeyType, ValueType, bInAllowDuplicateKeys>::KeyInitType KeyInitType;

	static FORCEINLINE bool Matches(KeyInitType A, KeyInitType B)
	{
		return A.HasSameIndexAndSerialNumber(B);
	}

	static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key);
	}
};

template <typename T>
struct TCallTraits<TWeakObjectPtr<T>> : public TCallTraitsBase<TWeakObjectPtr<T>>
{
	using ConstPointerType = TWeakObjectPtr<const T>;
};

/** Utility function to fill in a TArray<ClassName*> from a TArray<TWeakObjectPtr<ClassName>> */
template<typename DestArrayType, typename SourceArrayType>
void CopyFromWeakArray(DestArrayType& Dest, const SourceArrayType& Src)
{
	const int32 Count = Src.Num();
	Dest.Empty(Count);
	for (int32 Index = 0; Index < Count; Index++)
	{
		if (auto Value = Src[Index].Get())
		{
			Dest.Add(Value);
		}
	}
}

/** Utility function to fill in a TArray<TWeakObjectPtr<ClassName>> from a TArray<TObjectPtr<ClassName>> or TArray<ClassName*> */
template<typename DestArrayType, typename SourceArrayType>
void CopyToWeakArray(DestArrayType& Dest, const SourceArrayType& Src)
{
	const int32 Count = Src.Num();
	Dest.Empty(Count);
	for (int32 Index = 0; Index < Count; Index++)
	{
		if (auto* Value = Src[Index])
		{
			Dest.Add(Value);
		}
	}
}

/** Hash function. */
template <typename T>
[[nodiscard]] FORCEINLINE uint32 GetTypeHash(const TWeakObjectPtr<T>& WeakObjectPtr)
{
	return WeakObjectPtr.GetWeakPtrTypeHash();
}


/**
* Weak object pointer serialization, this forwards to FArchive::operator<<(struct FWeakObjectPtr&) or an override
*/
template<class T, class TWeakObjectPtrBase>
FArchive& operator<<( FArchive& Ar, TWeakObjectPtr<T, TWeakObjectPtrBase>& WeakObjectPtr )
{
	WeakObjectPtr.Serialize(Ar);
	return Ar;
}
