// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_exports

class UObject;

/**
 * FWeakObjectPtr is a weak pointer to a UObject. 
 * It can return nullptr later if the object is garbage collected.
 * It has no impact on if the object is garbage collected or not.
 * It can't be directly used across a network.
 *
 * Most often it is used when you explicitly do NOT want to prevent something from being garbage collected.
 */
struct FWeakObjectPtr;

template<class T=UObject, class TWeakObjectPtrBase=FWeakObjectPtr>
struct TWeakObjectPtr;

template <typename T> struct TIsPODType;
template <typename T> struct TIsWeakPointerType;
template <typename T> struct TIsZeroConstructType;

template<class T> struct TIsPODType<TWeakObjectPtr<T> > { enum { Value = true }; };
template<class T> struct TIsZeroConstructType<TWeakObjectPtr<T> > { enum { Value = true }; };
template<class T> struct TIsWeakPointerType<TWeakObjectPtr<T> > { enum { Value = true }; };

// IWYU pragma: end_exports
