// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	UnixPlatform.h: Setup for the linux platform
==================================================================================*/

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include <linux/version.h>
#include <signal.h>
#include "Clang/ClangPlatform.h"

struct FGenericPlatformTypes;

#define PLATFORM_UNIX_USE_CHAR16 1 UE_DEPRECATED_MACRO(5.7, "PLATFORM_UNIX_USE_CHAR16 has been deprecated and should be replaced with 1.")

/**
* Unix specific types
**/
struct FUnixPlatformTypes : public FGenericPlatformTypes
{
	typedef unsigned int		DWORD;
	typedef __SIZE_TYPE__		SIZE_T;
	typedef decltype(__null)	TYPE_OF_NULL;
	typedef char16_t			WIDECHAR;
	typedef WIDECHAR			TCHAR;
};

typedef FUnixPlatformTypes FPlatformTypes;

#define UNIX_MAX_PATH				PATH_MAX

#define PLATFORM_CAN_SUPPORT_EDITORONLY_DATA	1

// Base defines, defaults are commented out

#define PLATFORM_SEH_EXCEPTIONS_DISABLED				1
#define PLATFORM_LITTLE_ENDIAN							1
#define PLATFORM_SUPPORTS_UNALIGNED_LOADS				1
#define PLATFORM_COMPILER_DISTINGUISHES_INT_AND_LONG	1
#define PLATFORM_SUPPORTS_PRAGMA_PACK					1
#define PLATFORM_USE_LS_SPEC_FOR_WIDECHAR				1
#define PLATFORM_WCHAR_IS_4_BYTES						1
#define PLATFORM_TCHAR_IS_CHAR16						1
#define PLATFORM_HAS_BSD_TIME							1
#define PLATFORM_HAS_BSD_THREAD_CPUTIME					1
#define PLATFORM_USE_PTHREADS							1
#define PLATFORM_MAX_FILEPATH_LENGTH_DEPRECATED			UNIX_MAX_PATH /* @todo linux: avoid using PATH_MAX as it is known to be broken */
#define PLATFORM_HAS_NO_EPROCLIM						1
#define PLATFORM_HAS_BSD_IPV6_SOCKETS					1
#define PLATFORM_HAS_BSD_SOCKET_FEATURE_IOCTL			1
#define PLATFORM_HAS_BSD_SOCKET_FEATURE_POLL			1
#define PLATFORM_HAS_BSD_SOCKET_FEATURE_MSG_DONTWAIT	1
#define PLATFORM_HAS_BSD_SOCKET_FEATURE_RECVMMSG		1
#define PLATFORM_HAS_BSD_SOCKET_FEATURE_TIMESTAMP		1
#define PLATFORM_SUPPORTS_MIMALLOC						1
#define PLATFORM_SUPPORTS_STACK_SYMBOLS					1
#define PLATFORM_IS_ANSI_MALLOC_THREADSAFE				1
#define PLATFORM_ALLOW_ALLOCATIONS_IN_FASYNCWRITER_SERIALIZEBUFFERTOARCHIVE 0
#define PLATFORM_USE_ANSI_POSIX_MALLOC					1
#define PLATFORM_SUPPORTS_BINDLESS_RENDERING			1
#define PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS		1
#define PLATFORM_SUPPORTS_MESH_SHADERS					1


#define PLATFORM_RETURN_ADDRESS_FOR_CALLSTACKTRACING	PLATFORM_RETURN_ADDRESS

#if WITH_EDITOR
#define PLATFORM_FILE_READER_BUFFER_SIZE				(256*1024)
#endif

#if PLATFORM_CPU_X86_FAMILY
	#define PLATFORM_BREAK()							{ PRAGMA_DISABLE_CPP20_EXTENSION_WARNINGS __asm__ volatile("int $0x03"); PRAGMA_ENABLE_CPP20_EXTENSION_WARNINGS }
#else
	#define PLATFORM_BREAK()							raise(SIGTRAP)
#endif // PLATFORM_CPU_X86_FAMILY

#define PLATFORM_CODE_SECTION(Name)						__attribute__((section(Name)))

// SOCK_CLOEXEC is available on Unix since 2.6.27
#if LINUX_VERSION_CODE >= KERNEL_VERSION(2,6,27)
	#define PLATFORM_HAS_BSD_SOCKET_FEATURE_CLOSE_ON_EXEC	1
#endif // LINUX_VERSION_CODE >= KERNEL_VERSION(2,6,27)

#if defined(_M_IX86) || defined(__i386__) || defined(_M_X64) || defined(__x86_64__) || defined (__amd64__) 
	#define PLATFORM_ENABLE_VECTORINTRINSICS		1
#endif // defined(_M_IX86) || defined(__i386__) || defined(_M_X64) || defined(__x86_64__) || defined (__amd64__) 

#if PLATFORM_LINUXARM64
	// Enable NEON intrinsics for ARM64 builds
	#define PLATFORM_ENABLE_VECTORINTRINSICS_NEON	1
	#define PLATFORM_ENABLE_VECTORINTRINSICS		1
#endif

#ifndef PLATFORM_ALWAYS_HAS_FMA3
	#define PLATFORM_ALWAYS_HAS_FMA3							0
#endif


// Function type macros.
#define VARARGS															/* Functions with variable arguments */
#define CDECL															/* Standard C function */
#define STDCALL															/* Standard calling convention */
#if UE_BUILD_DEBUG
	#define FORCEINLINE inline 											/* Don't force code to be inline, or you'll run into -Wignored-attributes */
#else
	#define FORCEINLINE inline __attribute__ ((always_inline))			/* Force code to be inline */
#endif // UE_BUILD_DEBUG
#define FORCENOINLINE __attribute__((noinline))							/* Force code to NOT be inline */
#define FUNCTION_CHECK_RETURN_END __attribute__ ((warn_unused_result))	/* Wrap a function signature in this to warn that callers should not ignore the return value. */
#define FUNCTION_NO_RETURN_END __attribute__ ((noreturn))				/* Wrap a function signature in this to indicate that the function never returns. */

// Optimization macros (uses _Pragma to enable inside a #define).
#if (__clang_major__ > 3 || (__clang_major__ == 3 && __clang_minor__ >= 6))
	#define PRAGMA_DISABLE_OPTIMIZATION_ACTUAL _Pragma("clang optimize off")
	#define PRAGMA_ENABLE_OPTIMIZATION_ACTUAL  _Pragma("clang optimize on")
#endif

#define ABSTRACT abstract

// DLL export and import definitions
#define DLLEXPORT			__attribute__((visibility("default")))
#define DLLIMPORT			__attribute__((visibility("default")))

// Alignment.
#define GCC_PACK(n)			__attribute__((packed,aligned(n)))
#define GCC_ALIGN(n)		__attribute__((aligned(n)))
#if defined(__arm__)
#define REQUIRES_ALIGNED_ACCESS					1
#endif

// operator new/delete operators
// As of 10.9 we need to use _NOEXCEPT & cxx_noexcept compatible definitions
#if __has_feature(cxx_noexcept)
#define OPERATOR_NEW_THROW_SPEC
#else
#define OPERATOR_NEW_THROW_SPEC throw (std::bad_alloc)
#endif
#define OPERATOR_DELETE_THROW_SPEC noexcept
#define OPERATOR_NEW_NOTHROW_SPEC  noexcept
#define OPERATOR_DELETE_NOTHROW_SPEC  noexcept
