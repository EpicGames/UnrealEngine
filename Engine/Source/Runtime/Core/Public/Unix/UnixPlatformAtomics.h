// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	LinuxPlatformAtomics.h: Android platform Atomics functions
==============================================================================================*/

#pragma once

#include "Clang/ClangPlatformAtomics.h"

typedef FClangPlatformAtomics FPlatformAtomics;
