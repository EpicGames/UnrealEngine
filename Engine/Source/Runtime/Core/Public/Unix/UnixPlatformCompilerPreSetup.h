// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef PRAGMA_DISABLE_DEPRECATED_REGISTER_WARNINGS
	#define PRAGMA_DISABLE_DEPRECATED_REGISTER_WARNINGS \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wdeprecated-register\"")
#endif // PRAGMA_DISABLE_DEPRECATED_REGISTER_WARNINGS

#ifndef PRAGMA_ENABLE_DEPRECATED_REGISTER_WARNINGS
	#define PRAGMA_ENABLE_DEPRECATED_REGISTER_WARNINGS \
		_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_DEPRECATED_REGISTER_WARNINGS

#ifndef PRAGMA_DISABLE_NULL_DEREFERENCE_WARNINGS
	#if (__clang_major__ > 3 || (__clang_major__ == 3 && __clang_minor__ >= 9))
		#define PRAGMA_DISABLE_NULL_DEREFERENCE_WARNINGS \
				_Pragma ("clang diagnostic push") \
				_Pragma ("clang diagnostic ignored \"-Wnull-dereference\"")
	#else
		#define PRAGMA_DISABLE_NULL_DEREFERENCE_WARNINGS \
				_Pragma ("clang diagnostic push")
	#endif // (__clang_major__ > 3 || (__clang_major__ == 3 && __clang_minor__ >= 9))
#endif // PRAGMA_DISABLE_NULL_DEREFERENCE_WARNINGS

#ifndef PRAGMA_ENABLE_NULL_DEREFERENCE_WARNINGS
	#define PRAGMA_ENABLE_NULL_DEREFERENCE_WARNINGS \
			_Pragma("clang diagnostic pop")
#endif // PRAGMA_ENABLE_NULL_DEREFERENCE_WARNINGS

// Disable platform-specific CA warnings around SDK includes in addition to common Clang warnings
#ifndef THIRD_PARTY_INCLUDES_START
	#define THIRD_PARTY_INCLUDES_START \
		UE_COMPILER_THIRD_PARTY_INCLUDES_START \
		PRAGMA_DISABLE_NULL_DEREFERENCE_WARNINGS \
		PRAGMA_DISABLE_DEPRECATED_REGISTER_WARNINGS \
		PRAGMA_DISABLE_MACRO_REDEFINED_WARNINGS
#endif

#ifndef THIRD_PARTY_INCLUDES_END
	#define THIRD_PARTY_INCLUDES_END \
		PRAGMA_ENABLE_MACRO_REDEFINED_WARNINGS \
		PRAGMA_ENABLE_DEPRECATED_REGISTER_WARNINGS \
		PRAGMA_ENABLE_NULL_DEREFERENCE_WARNINGS \
		UE_COMPILER_THIRD_PARTY_INCLUDES_END
#endif

// Make certain warnings always be warnings, even despite -Werror.
// Rationale: we don't want to suppress those as there are plans to address them (e.g. UE-12341), but breaking builds due to these warnings is very expensive
// since they cannot be caught by all compilers that we support. They are deemed to be relatively safe to be ignored, at least until all SDKs/toolchains start supporting them.
#pragma clang diagnostic warning "-Wparentheses-equality"

#include "Clang/ClangPlatformCompilerPreSetup.h"
