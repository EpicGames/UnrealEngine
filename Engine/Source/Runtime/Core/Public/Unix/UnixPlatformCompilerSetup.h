// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	UnixPlatformCompilerSetup.h: pragmas, version checks and other things for the Unix compiler
==============================================================================================*/

#pragma once
